// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define WhiteStandard uint
static const WhiteStandard WhiteStandard_D65 = 0;
static const WhiteStandard WhiteStandard_D60 = 1;
static const WhiteStandard WhiteStandard_DCI = 2;

struct FColorSpace
{
	WhiteStandard White;
	float3x3 XYZtoRGB;
	float3x3 RGBtoXYZ;
};

static const float3x3 AP0_2_XYZ_MAT =
{
	0.9525523959, 0.0000000000, 0.0000936786,
	0.3439664498, 0.7281660966,-0.0721325464,
	0.0000000000, 0.0000000000, 1.0088251844,
};

static const float3x3 XYZ_2_AP0_MAT =
{
	 1.0498110175, 0.0000000000,-0.0000974845,
	-0.4959030231, 1.3733130458, 0.0982400361,
	 0.0000000000, 0.0000000000, 0.9912520182,
};

static const float3x3 AP1_2_XYZ_MAT =
{
	 0.6624541811, 0.1340042065, 0.1561876870,
	 0.2722287168, 0.6740817658, 0.0536895174,
	-0.0055746495, 0.0040607335, 1.0103391003,
};

static const float3x3 XYZ_2_AP1_MAT =
{
	 1.6410233797, -0.3248032942, -0.2364246952,
	-0.6636628587,  1.6153315917,  0.0167563477,
	 0.0117218943, -0.0082844420,  0.9883948585,
};

static const float3x3 AP0_2_AP1_MAT = //mul( AP0_2_XYZ_MAT, XYZ_2_AP1_MAT );
{
	 1.4514393161, -0.2365107469, -0.2149285693,
	-0.0765537734,  1.1762296998, -0.0996759264,
	 0.0083161484, -0.0060324498,  0.9977163014,
};

static const float3x3 AP1_2_AP0_MAT = //mul( AP1_2_XYZ_MAT, XYZ_2_AP0_MAT );
{
	 0.6954522414,  0.1406786965,  0.1638690622,
	 0.0447945634,  0.8596711185,  0.0955343182,
	-0.0055258826,  0.0040252103,  1.0015006723,
};

static const float3 AP1_RGB2Y =
{
	0.2722287168, //AP1_2_XYZ_MAT[0][1],
	0.6740817658, //AP1_2_XYZ_MAT[1][1],
	0.0536895174, //AP1_2_XYZ_MAT[2][1]
};

// REC 709 primaries
static const float3x3 XYZ_2_sRGB_MAT =
{
	 3.2409699419, -1.5373831776, -0.4986107603,
	-0.9692436363,  1.8759675015,  0.0415550574,
	 0.0556300797, -0.2039769589,  1.0569715142,
};

static const float3x3 sRGB_2_XYZ_MAT =
{
	0.4123907993, 0.3575843394, 0.1804807884,
	0.2126390059, 0.7151686788, 0.0721923154,
	0.0193308187, 0.1191947798, 0.9505321522,
};

// REC 2020 primaries
static const float3x3 XYZ_2_Rec2020_MAT =
{
	 1.7166511880, -0.3556707838, -0.2533662814,
	-0.6666843518,  1.6164812366,  0.0157685458,
	 0.0176398574, -0.0427706133,  0.9421031212,
};

static const float3x3 Rec2020_2_XYZ_MAT =
{
	0.6369580483, 0.1446169036, 0.1688809752,
	0.2627002120, 0.6779980715, 0.0593017165,
	0.0000000000, 0.0280726930, 1.0609850577,
};

// P3, D65 primaries
static const float3x3 XYZ_2_P3D65_MAT =
{
	 2.4934969119, -0.9313836179, -0.4027107845,
	-0.8294889696,  1.7626640603,  0.0236246858,
	 0.0358458302, -0.0761723893,  0.9568845240,
};

static const float3x3 P3D65_2_XYZ_MAT =
{
	0.4865709486, 0.2656676932, 0.1982172852,
	0.2289745641, 0.6917385218, 0.0792869141,
	0.0000000000, 0.0451133819, 1.0439443689,
};

// Bradford chromatic adaptation transforms between ACES white point (D60) and sRGB white point (D65)
static const float3x3 D65_2_D60_CAT =
{
	 1.0130349146, 0.0061052578, -0.0149709436,
	 0.0076982301, 0.9981633521, -0.0050320385,
	-0.0028413174, 0.0046851567,  0.9245061375,
};

static const float3x3 D60_2_D65_CAT =
{
	 0.9872240087, -0.0061132286, 0.0159532883,
	-0.0075983718,  1.0018614847, 0.0053300358,
	 0.0030725771, -0.0050959615, 1.0816806031,
};

static const float3x3 CAM16_2_XYZ_MAT =
{
	 2.0512756811, -1.1400313439,  0.0887556628,
	 0.4269389763,  0.7005835277, -0.1275225040,
	-0.0174712779, -0.0384725929,  1.0589468739
};

static const float3x3 XYZ_2_CAM16_MAT =
{
	 0.3640744835,  0.5947008156, 0.04110127349,
	-0.2222450987,  1.0738554823, 0.14794533610,
	-0.0020676190,  0.0488260453, 0.95038755696
};

float rgb_2_saturation(float3 rgb)
{
	float minrgb = min(min(rgb.r, rgb.g), rgb.b);
	float maxrgb = max(max(rgb.r, rgb.g), rgb.b);
	return (max(maxrgb, 1e-10) - max(minrgb, 1e-10)) / max(maxrgb, 1e-2);
}

// Transformations from RGB to other color representations
float rgb_2_hue(float3 rgb)
{
	// Returns a geometric hue angle in degrees (0-360) based on RGB values.
	// For neutral colors, hue is undefined and the function will return a quiet NaN value.
	float hue;
	if (rgb[0] == rgb[1] && rgb[1] == rgb[2]) {
		//hue = FLT_NAN; // RGB triplets where RGB are equal have an undefined hue
		hue = 0;
	}
	else {
		hue = (180. / PI) * atan2(sqrt(3.0) * (rgb[1] - rgb[2]), 2 * rgb[0] - rgb[1] - rgb[2]);
	}

	if (hue < 0.) hue = hue + 360;

	return clamp(hue, 0, 360);
}

float center_hue(float hue, float centerH)
{
	float hueCentered = hue - centerH;
	if (hueCentered < -180.) hueCentered += 360;
	else if (hueCentered > 180.) hueCentered -= 360;
	return hueCentered;
}

// Transformations between CIE XYZ tristimulus values and CIE x,y 
// chromaticity coordinates
float3 XYZ_2_xyY(float3 XYZ)
{
	float3 xyY;
	float divisor = (XYZ[0] + XYZ[1] + XYZ[2]);
	if (divisor == 0.) divisor = 1e-10;
	xyY[0] = XYZ[0] / divisor;
	xyY[1] = XYZ[1] / divisor;
	xyY[2] = XYZ[1];

	return xyY;
}

float3 xyY_2_XYZ(float3 xyY)
{
	float3 XYZ;
	XYZ[0] = xyY[0] * xyY[2] / max(xyY[1], 1e-10);
	XYZ[1] = xyY[2];
	XYZ[2] = (1.0 - xyY[0] - xyY[1]) * xyY[2] / max(xyY[1], 1e-10);

	return XYZ;
}


float3x3 ChromaticAdaptation( float2 src_xy, float2 dst_xy )
{
	// Von Kries chromatic adaptation 

	// Bradford
	const float3x3 ConeResponse =
	{
		 0.8951,  0.2664, -0.1614,
		-0.7502,  1.7135,  0.0367,
		 0.0389, -0.0685,  1.0296,
	};
	const float3x3 InvConeResponse =
	{
		 0.9869929, -0.1470543,  0.1599627,
		 0.4323053,  0.5183603,  0.0492912,
		-0.0085287,  0.0400428,  0.9684867,
	};

	float3 src_XYZ = xyY_2_XYZ( float3( src_xy, 1 ) );
	float3 dst_XYZ = xyY_2_XYZ( float3( dst_xy, 1 ) );

	float3 src_coneResp = mul( ConeResponse, src_XYZ );
	float3 dst_coneResp = mul( ConeResponse, dst_XYZ );

	float3x3 VonKriesMat =
	{
		{ dst_coneResp[0] / src_coneResp[0], 0.0, 0.0 },
		{ 0.0, dst_coneResp[1] / src_coneResp[1], 0.0 },
		{ 0.0, 0.0, dst_coneResp[2] / src_coneResp[2] }
	};

	return mul( InvConeResponse, mul( VonKriesMat, ConeResponse ) );
}

// ------- Glow module functions
float glow_fwd(float ycIn, float glowGainIn, float glowMid)
{
	float glowGainOut;

	if (ycIn <= 2. / 3. * glowMid) {
		glowGainOut = glowGainIn;
	}
	else if (ycIn >= 2 * glowMid) {
		glowGainOut = 0;
	}
	else {
		glowGainOut = glowGainIn * (glowMid / ycIn - 0.5);
	}

	return glowGainOut;
}

float glow_inv(float ycOut, float glowGainIn, float glowMid)
{
	float glowGainOut;

	if (ycOut <= ((1 + glowGainIn) * 2. / 3. * glowMid)) {
		glowGainOut = -glowGainIn / (1 + glowGainIn);
	}
	else if (ycOut >= (2. * glowMid)) {
		glowGainOut = 0.;
	}
	else {
		glowGainOut = glowGainIn * (glowMid / ycOut - 1. / 2.) / (glowGainIn / 2. - 1.);
	}

	return glowGainOut;
}

float rgb_2_yc(float3 rgb, float ycRadiusWeight = 1.75)
{
	// Converts RGB to a luminance proxy, here called YC
	// YC is ~ Y + K * Chroma
	// Constant YC is a cone-shaped surface in RGB space, with the tip on the 
	// neutral axis, towards white.
	// YC is normalized: RGB 1 1 1 maps to YC = 1
	//
	// ycRadiusWeight defaults to 1.75, although can be overridden in function 
	// call to rgb_2_yc
	// ycRadiusWeight = 1 -> YC for pure cyan, magenta, yellow == YC for neutral 
	// of same value
	// ycRadiusWeight = 2 -> YC for pure red, green, blue  == YC for  neutral of 
	// same value.

	float r = rgb[0];
	float g = rgb[1];
	float b = rgb[2];

	float chroma = sqrt(b * (b - g) + g * (g - r) + r * (r - b));

	return (b + g + r + ycRadiusWeight * chroma) / 3.;
}

float sigmoid_shaper(float x)
{
	// Sigmoid function in the range 0 to 1 spanning -2 to +2.

	float t = max(1 - abs(0.5 * x), 0);
	float y = 1 + sign(x) * (1 - t * t);
	return 0.5 * y;
}

