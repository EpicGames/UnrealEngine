// Copyright Epic Games, Inc. All Rights Reserved.

// <ACEStransformID>urn:ampas:aces:transformId:v2.0:Lib.Academy.DisplayEncoding.a2.v1</ACEStransformID>
// <ACESuserName>Display Encoding Functions</ACESuserName>

// 
// Library File with functions used for the display encoding/decoding steps
//

#pragma once

#include "ACESOutputTransform.ush"

float3 apply_white_scale( float3 XYZluminance, 
                            ODTParams PARAMS, 
                            bool _invert)
{
    // If the creative white differs from the calibration white of the display,
    // unequal display code values will be required to produce the neutral of
    // the creative white. Without scaling, one channel would hit the max value
    // first while the other channels continue to increase, resulting in a hue
    // shift. To avoid this, the white scaling finds the largest channel and
    // applies a scale factor to force the point where this channel hits max to
    // 1.0, assuring that all three channels "fit" within the peak value. In the
    // inverse direction, the white scaling is removed.

    float3 RGB_w = mul( PARAMS.OUTPUT_XYZ_TO_RGB, PARAMS.XYZ_w_limit );
    float3 RGB_w_f = (1./referenceLuminance) * RGB_w;
    float scale = 1. / max( max(RGB_w_f[0], RGB_w_f[1]), RGB_w_f[2]); 
    // scale factor is equal to 1/largestChannel

    return (_invert ? (1./scale) : scale) * XYZluminance;
}

float3 clamp_zero_to_peakLuminance( float3 XYZ_in,
                                    ODTParams PARAMS)
{
    float3 rgb = mul(PARAMS.LIMIT_XYZ_TO_RGB, XYZ_in);

    // Clamp to relative peakLuminance in RGB space prior to white scaling
    rgb = clamp(rgb, 0.0, PARAMS.peakLuminance / referenceLuminance);

    float3 XYZ = mul(PARAMS.LIMIT_RGB_TO_XYZ, rgb);

    return XYZ;
}
