// Copyright Epic Games, Inc. All Rights Reserved.

// <ACEStransformID>urn:ampas:aces:transformId:v2.0:Lib.Academy.OutputTransform.a2.v1</ACEStransformID>
// <ACESuserName>Output Transform Functions</ACESuserName>

//
// Library File with functions and presets used for the forward and inverse output
// transform
//

#pragma once

#include "ACESUtilities.ush"
#include "ACESTonescale.ush"

// Gamut compression constants
static const float smoothCusps = 0.12;
static const float smoothM = 0.27;
static const float cuspMidBlend = 1.3;

static const float focusGainBlend = 0.3;
static const float focusAdjustGain = 0.55;
static const float focusDistance = 1.35;
static const float focusDistanceScaling = 1.75;

// Constant used in gamut mapping
static const float compressionThreshold = 0.75;

static const int tableSize = 360;       // add 1 extra entry at end that is to duplicate first entry for wrapped hue
static const int additionalTableEntries = 2; // allots for extra entries to wrap the hues without special cases
static const int totalTableSize = tableSize + additionalTableEntries;
static const int baseIndex = 1; // array index for smallest hue, which is not necessarily a 0.0 hue angle

static const float gammaMinimum = 0.0;
static const float gammaMaximum = 5.0;
static const float gammaSearchStep = 0.4;
static const float gammaAccuracy = 1e-5;

static const float referenceLuminance = 100.;

// CAM Parameters
static const float L_A = 100.;
static const float Y_b = 20.;

static const float ac_resp = 1.0;
static const float ra = 2. * ac_resp;
static const float ba = 0.05 + (2. - ra);

static const float3 surround = float3(0.9, 0.59, 0.9);

// Matrix for Hellwig inverse
static const float3x3 panlrcm =
{
	460.0,  451.0,  288.0 ,
	460.0, -891.0, -261.0 ,
	460.0, -220.0, -6300.0
};

struct ODTParams
{
    float peakLuminance;

    // Tonescale // Set via TSParams structure
    float n_r; // normalized white in nits (what 1.0 should be)
    float g;   // surround / contrast
    float t_1; // shadow toe or flare/glare compensation
    float c_t;
    float s_2;
    float u_2;
    float m_2;

    // Chroma Compression
    float limitJmax;
    float midJ;
    float model_gamma;
    float sat;
    float sat_thr;
    float compr;
    float chromaCompressScale;

    float focusDist;

    // Limit
    float3x3 LIMIT_RGB_TO_XYZ;
    float3x3 LIMIT_XYZ_TO_RGB;
    float3 XYZ_w_limit;

    // Output
    float3x3 OUTPUT_RGB_TO_XYZ;
    float3x3 OUTPUT_XYZ_TO_RGB;
    float3 XYZ_w_output;

    float lowerHullGamma;

    Texture2D<float> reach_table;
    Texture2D<float> gamut_table;
    Texture2D<float> gamma_table;
};

float wrap_to_360(float hue)
{
    float y = fmod(hue, 360.);
    if (y < 0.)
    {
        y = y + 360.;
    }
    return y;
}

int hue_position_in_uniform_table(float hue, int table_size)
{
    const float wrapped_hue = wrap_to_360(hue);
    int result = (wrapped_hue / 360. * table_size);
    return result;
}

int next_position_in_table(int entry, int table_size)
{
    int result = (entry + 1) % table_size;
    return result;
}

float base_hue_for_position(int i_lo, int table_size)
{
    float result = i_lo * 360. / table_size;
    return result;
}

// CAM Functions
float3 post_adaptation_non_linear_response_compression_forward(float3 RGB, float F_L)
{
    float3 F_L_RGB = spow(F_L * abs(RGB) / 100., 0.42);
    float3 RGB_c = (400. * copysign(1., RGB) * F_L_RGB) / (27.13 + F_L_RGB);

    return RGB_c;
}

float3 post_adaptation_non_linear_response_compression_inverse(float3 RGB, float F_L)
{
    float3 RGB_p = sign(RGB) * 100. / F_L * spow((27.13 * abs(RGB)) / (400. - abs(RGB)), 1.0 / 0.42);

    return RGB_p;
}

float3 XYZ_to_Hellwig2022_JMh(  float3 XYZ,
                                float3 XYZ_w)
{

    float Y_w = XYZ_w[1];

    // Step 0 - Converting CIE XYZ tristimulus values to sharpened RGB values
    float3 RGB_w = mul(XYZ_2_CAM16_MAT, XYZ_w);

    // Ignore degree of adaptation.
    float D = 1.0;

    // Viewing condition dependent parameters
    float k = 1. / (5. * L_A + 1.);
    float k4 = pow(k, 4.);
    float F_L = 0.2 * k4 * (5. * L_A) + 0.1 * pow((1. - k4), 2.) * spow(5. * L_A, 1. / 3.);
    float n = Y_b / Y_w;
    float z = 1.48 + sqrt(n);

    float3 D_RGB = D * Y_w / RGB_w + 1. - D;

    float3 RGB_wc = D_RGB * RGB_w;

    float3 RGB_aw = post_adaptation_non_linear_response_compression_forward(RGB_wc, F_L);
    float A_w = ra * RGB_aw[0] + RGB_aw[1] + ba * RGB_aw[2];

    // Step 1 - Converting CIE XYZ tristimulus values to sharpened RGB values
    float3 RGB = mul(XYZ_2_CAM16_MAT, XYZ);

    // Step 2
    float3 RGB_c = D_RGB * RGB;

    // Step 3 - apply  forward post-adaptation non-linear response compression
    float3 RGB_a = post_adaptation_non_linear_response_compression_forward(RGB_c, F_L);

    // Step 4 - Converting to preliminary cartesian  coordinates
    float a = RGB_a[0] - 12. * RGB_a[1] / 11. + RGB_a[2] / 11.;
    float b = (RGB_a[0] + RGB_a[1] - 2. * RGB_a[2]) / 9.;

    // Computing the hue angle math h
    // Unclear why this isn't matching the python version
    float hr = satan2(b, a);
    float h = wrap_to_360(radians_to_degrees(hr));

    // Step 6 - Computing achromatic responses for the stimulus
    float A = ra * RGB_a[0] + RGB_a[1] + ba * RGB_a[2];

    // Step 7 - Computing the correlate of lightness, J
    float J = 100. * pow(A / A_w, surround[1] * z);

    // Step 9 - Computing the correlate of colourfulness, M
    float M = (J == 0.0) ? 0.0 : 43. * surround[2] * sqrt(a * a + b * b);

    float3 return_JMh = float3(J, M, h);

    return return_JMh;
}

float3 Hellwig2022_JMh_to_XYZ(  float3 JMh,
                                float3 XYZ_w)
{
    float J = JMh[0];
    float M = JMh[1];
    float h = JMh[2];

    float Y_w = XYZ_w[1];

    // Step 0 - Converting CIE XYZ tristimulus values to sharpened RGB values.
    float3 RGB_w = mul(XYZ_2_CAM16_MAT, XYZ_w);

    // Ignore degree of adaptation.
    float D = 1.0;

    // Viewing condition dependent parameters
    float k = 1.0 / (5.0 * L_A + 1.0);
    float k4 = pow(k, 4.0);
    float F_L = 0.2 * k4 * (5.0 * L_A) + 0.1 * pow((1.0 - k4), 2.0) * spow(5.0 * L_A, 1.0 / 3.0);
    float n = sdiv(Y_b, Y_w);
    float z = 1.48 + sqrt(n);

    float3 D_RGB = D * Y_w / RGB_w + 1.0 - D;

    float3 RGB_wc = D_RGB * RGB_w;

    float3 RGB_aw = post_adaptation_non_linear_response_compression_forward(RGB_wc, F_L);

    float A_w = ra * RGB_aw[0] + RGB_aw[1] + ba * RGB_aw[2];

    float hr = degrees_to_radians(h);

    // No Helmholtz-Kohlrausch effect

    // Computing achromatic respons A for the stimulus
    float A = A_w * spow(J / 100.0, 1.0 / (surround[1] * z));

    // Computing P_p_1 to P_p_2
    float P_p_1 = 43.0 * surround[2];
    float P_p_2 = A;

    // Step 3 - Computing opponent colour dimensions a and b
    float gamma = M / P_p_1;
    float a = gamma * cos(hr);
    float b = gamma * sin(hr);

    // Step 4 - Applying post-adaptation non-linear response compression matrix
    float3 vec = {P_p_2, a, b};
    float3 RGB_a = (1.0 / 1403.0) * mul(panlrcm, vec);

    // Step 5 - Applying the inverse post-adaptation non-linear respnose compression
    float3 RGB_c = post_adaptation_non_linear_response_compression_inverse(RGB_a, F_L);

    // Step 6
    float3 RGB = RGB_c / D_RGB;

    // Step 7
    float3 XYZ = mul(CAM16_2_XYZ_MAT, RGB);

    return XYZ;
}

float Hellwig_J_to_Y(float J,
                     float surround = 0.59,
                     float L_A = 100.,
                     float Y_b = 20.)
{
    // Viewing conditions dependent parameters (could be pre-calculated)
    float k = 1.0 / (5. * L_A + 1.);
    float k4 = k * k * k * k;
    float F_L = 0.2 * k4 * (5. * L_A) + 0.1 * pow((1. - k4), 2.) * pow(5. * L_A, 1. / 3.);
    float n = Y_b / 100.;
    float z = 1.48 + sqrt(n);
    float F_L_W = pow(F_L, 0.42);
    float A_w = (400. * F_L_W) / (27.13 + F_L_W);

    float A = A_w * pow(abs(J) / 100., 1. / (surround * z));

    return sign(J) * 100. / F_L * pow((27.13 * A) / (400.0 - A), 1. / 0.42);
}

float Y_to_Hellwig_J(float Y,
                     float surround = 0.59,
                     float L_A = 100.,
                     float Y_b = 20.)
{
    // Viewing conditions dependent parameters (could be pre-calculated)
    float k = 1.0 / (5. * L_A + 1.);
    float k4 = k * k * k * k;
    float F_L = 0.2 * k4 * (5. * L_A) + 0.1 * pow((1. - k4), 2.) * pow(5. * L_A, 1. / 3.);
    float n = Y_b / 100.;
    float z = 1.48 + sqrt(n);
    float F_L_W = pow(F_L, 0.42);
    float A_w = (400. * F_L_W) / (27.13 + F_L_W);

    float F_L_Y = pow(F_L * abs(Y) / 100., 0.42);

    return sign(Y) * 100. * pow(((400. * F_L_Y) / (27.13 + F_L_Y)) / A_w, surround * z);
}

float3 clamp_AP1(float3 AP1, float peakLuminance)
{
    const float upper_clamp_limit = 8. * (128 + 768 * (log(peakLuminance/100.)/log(10000./100.))); 
    // limit to nice power of 2 (3 stops) above that needed to max out
    // note the quantity (128 + ...) is the definition of r_hit from the tonescale constants

    float3 ap1_clamped = clamp(AP1, 0., upper_clamp_limit);
    float3 XYZ_clamped = mul(AP1_2_XYZ_MAT, ap1_clamped);

    return XYZ_clamped;
}

float3 aces_to_JMh(float3 aces,
                     float peakLuminance)
{
    // AP0 to AP1
    float3 AP1 = mul(AP0_2_AP1_MAT, aces);

    // Clamp to half float range
    float3 XYZ = clamp_AP1(AP1, peakLuminance);

    // XYZ to JMh
    const float3 RGB_w = referenceLuminance;
    float3 XYZ_w = mul(AP0_2_XYZ_MAT, RGB_w);

    float3 XYZluminance = referenceLuminance * XYZ;
    float3 JMh = XYZ_to_Hellwig2022_JMh(XYZluminance, XYZ_w);

    return JMh;
}

float3 JMh_to_aces(float3 JMh,
                     float peakLuminance)
{
    const float3 RGB_w = referenceLuminance;
    float3 XYZ_w_aces = mul(AP0_2_XYZ_MAT, RGB_w);

    // JMh to XYZ
    float3 XYZluminance = Hellwig2022_JMh_to_XYZ(JMh, XYZ_w_aces);

    float3 XYZ = (1. / referenceLuminance) * XYZluminance;

    // XYZ to ACES
    float3 ACES = mul(XYZ_2_AP0_MAT, XYZ);

    return ACES;
}

ODTParams init_ODTParams(
    float peakLuminance,
    float3x3 LIMIT_RGB_TO_XYZ,
    float3x3 LIMIT_XYZ_TO_RGB,
    float3x3 OUTPUT_RGB_TO_XYZ,
    float3x3 OUTPUT_XYZ_TO_RGB,
    Texture2D<float> reach_table,
    Texture2D<float> gamut_table,
    Texture2D<float> gamma_table
)
{
    TSParams TSPARAMS = init_TSParams(peakLuminance);

    float limitJmax = Y_to_Hellwig_J(peakLuminance);
    float midJ = Y_to_Hellwig_J(TSPARAMS.c_t * 100.);

    // Chroma compress presets
    static const float chroma_compress = 2.4;
    static const float chroma_compress_fact = 3.3;
    static const float chroma_expand = 1.3;
    static const float chroma_expand_fact = 0.69;
    static const float chroma_expand_thr = 0.5;

    // Calculated chroma compress variables
    const float log_peak = log10(TSPARAMS.n / TSPARAMS.n_r);
    const float compr = chroma_compress + (chroma_compress * chroma_compress_fact) * log_peak;
    const float sat = max(0.2, chroma_expand - (chroma_expand * chroma_expand_fact) * log_peak);
    const float sat_thr = chroma_expand_thr / TSPARAMS.n;
    const float chromaCompressScale = pow(0.03379 * TSPARAMS.n, 0.30596) - 0.45135;

    const float model_gamma = 1. / (surround[1] * (1.48 + sqrt(Y_b / L_A)));

    const float focusDist = focusDistance +
                            focusDistance * focusDistanceScaling * log_peak;

    static const float3 RGB_w = float3(100., 100., 100.);

    const float lowerHullGamma = 1.14 + 0.07 * log_peak;

    // Limiting Primaries
    float3 XYZ_w_limit = mul(LIMIT_RGB_TO_XYZ, RGB_w);

    // Output / Encoding Primaries
    float3 XYZ_w_output = mul(OUTPUT_RGB_TO_XYZ, RGB_w);

    const ODTParams ODTPARAMS = {
        peakLuminance,

        // Tonescale
        TSPARAMS.n_r, // normalized white in nits (what 1.0 should be)
        TSPARAMS.g,   // surround / contrast
        TSPARAMS.t_1,
        TSPARAMS.c_t,
        TSPARAMS.s_2,
        TSPARAMS.u_2,
        TSPARAMS.m_2,

        // Chroma Compression
        limitJmax,
        midJ,
        model_gamma,
        sat,
        sat_thr,
        compr,
        chromaCompressScale,

        focusDist,

        // Limit
        LIMIT_RGB_TO_XYZ,
        LIMIT_XYZ_TO_RGB,
        XYZ_w_limit,

        // Output
        OUTPUT_RGB_TO_XYZ,
        OUTPUT_XYZ_TO_RGB,
        XYZ_w_output,

        lowerHullGamma, // lowerHullGamma

        reach_table,
        gamut_table,
        gamma_table
    };

    return ODTPARAMS;
}

// Chroma compression
//
// Compresses colors inside the gamut with the aim for colorfulness to have an
// appropriate rate of change from display black to display white, and from
// achromatic outward to purer colors.
//
float chromaCompressionNorm(float h,
                            ODTParams PARAMS)
{

    float hr = degrees_to_radians(h);

    float a = cos(hr);
    float b = sin(hr);
    float cos_hr2 = a * a - b * b;
    float sin_hr2 = 2.0 * a * b;
    float cos_hr3 = 4.0 * a * a * a - 3.0 * a;
    float sin_hr3 = 3.0 * b - 4.0 * b * b * b;

    float M = 11.34072 * a +
              16.46899 * cos_hr2 +
              7.88380 * cos_hr3 +
              14.66441 * b +
              -6.37224 * sin_hr2 +
              9.19364 * sin_hr3 +
              77.12896;

    return M * PARAMS.chromaCompressScale;
}

// A "toe" function that remaps the given value x between 0 and limit.
// The k1 and k2 parameters change the size and shape of the toe.
// https://www.desmos.com/calculator/6vplvw14ti
float toe_fwd(float x,
          float limit,
          float k1_in,
          float k2_in)
{
    if (x > limit)
        return x;

    const float k2 = max(k2_in, 0.001);
    const float k1 = sqrt(k1_in * k1_in + k2 * k2);
    const float k3 = (limit + k1) / (limit + k2);
    const float minus_b = k3 * x - k1;
    const float minus_c = k2 * k3 * x;

    return 0.5 * (minus_b + sqrt(minus_b * minus_b + 4. * minus_c));
}

float toe_inv(float x,
          float limit,
          float k1_in,
          float k2_in)
{
    if (x > limit)
        return x;

    float k2 = max(k2_in, 0.001);
    float k1 = sqrt(k1_in * k1_in + k2 * k2);
    float k3 = (limit + k1) / (limit + k2);

    return (x * x + k1 * x) / (k3 * (x + k2));
}

float reachMFromTable(float h,
                      Texture2D<float> reach_table)
{
    int i_lo = hue_position_in_uniform_table(h, tableSize);
    int i_hi = next_position_in_table(i_lo, tableSize);

    float t = (h - i_lo) / (i_hi - i_lo);

    return lerp(reach_table.Load(int3(i_lo, 0, 0)), reach_table.Load(int3(i_hi, 0, 0)), t);
}

// In-gamut chroma compression
//
// Compresses colors inside the gamut with the aim for colorfulness to have an
// appropriate rate of change from display black to display white, and from
// achromatic outward to purer colors.
float chromaCompression(float3 JMh,
                        float origJ,
                        ODTParams PARAMS,
                        bool _invert = false)
{
    float J = JMh[0];
    float M = JMh[1];
    float h = JMh[2];

    if (M == 0.0)
    {
        return M;
    }

    float nJ = J / PARAMS.limitJmax;
    float snJ = max(0., 1. - nJ);
    float Mnorm = chromaCompressionNorm(h, PARAMS);
    float limit = pow(nJ, PARAMS.model_gamma) * reachMFromTable(h, PARAMS.reach_table) / Mnorm;

    float toe_limit = limit - 0.001;
    float toe_snJ_sat = snJ * PARAMS.sat;
    float toe_sqrt_nJ_sat_thr = sqrt(nJ * nJ + PARAMS.sat_thr);
    float toe_nJ_compr = nJ * PARAMS.compr;

    if (!_invert) { // Forward chroma compression
        // Rescaling of M with the tonescaled J to get the M to the same range as
        // J after the tonescale.  The rescaling uses the Hellwig2022 model gamma to
        // keep the M/J ratio correct (keeping the chromaticities constant).
        M = M * pow(J / origJ, PARAMS.model_gamma);

        // Normalize M with the rendering space cusp M
        M = M / Mnorm;

        // Expand the colorfulness by running the toe function in reverse.  The goal is to
        // expand less saturated colors less and more saturated colors more.  The expansion
        // increases saturation in the shadows and mid-tones but not in the highlights.
        // The 0.001 offset starts the expansions slightly above zero.  The sat_thr makes
        // the toe less aggressive near black to reduce the expansion of noise.
        M = limit - toe_fwd(limit - M,
                        toe_limit,
                        toe_snJ_sat,
                        toe_sqrt_nJ_sat_thr);

        // Compress the colorfulness.  The goal is to compress less saturated colors more and
        // more saturated colors less, especially in the highlights.  This step creates the
        // saturation roll-off in the highlights, but attemps to preserve pure colors.  This
        // mostly affects highlights and mid-tones, and does not compress shadows.
        M = toe_fwd(M,
                limit,
                toe_nJ_compr,
                snJ);

        // Denormalize
        M = M * Mnorm;
    }
    else
    {
        M = M / Mnorm;
        M = toe_inv(M,
                limit,
                toe_nJ_compr,
                snJ);
        M = limit - toe_inv(limit - M,
                        toe_limit,
                        toe_snJ_sat,
                        toe_sqrt_nJ_sat_thr);
        M = M * Mnorm;
        M = M * pow(J / origJ, -PARAMS.model_gamma);
    }

    return M;
}

float compressionFunction(float v,
                          float thr,
                          float lim,
                          bool _invert = false)
{
    float s = (lim - thr) * (1.0 - thr) / (lim - 1.0);
    float nd = (v - thr) / s;

    float vCompressed;

    if (_invert)
    {
        if (v < thr || lim <= 1.0001 || v > thr + s)
        {
            vCompressed = v;
        }
        else
        {
            vCompressed = thr + s * (-nd / (nd - 1));
        }
    }
    else
    {
        if (v < thr || lim <= 1.0001)
        {
            vCompressed = v;
        }
        else
        {
            vCompressed = thr + s * nd / (1.0 + nd);
        }
    }

    return vCompressed;
}

int midpoint(int low, int high)
{
    return (low + high) / 2;
}

float2 cuspFromTable(float h, Texture2D<float> gamut_table)
{
    float3 lo;
    float3 hi;

    int low_i = 0;
    int high_i = baseIndex + tableSize; // allowed as we have an extra entry in the table
    int i = hue_position_in_uniform_table(h, tableSize) + baseIndex;

    while (low_i + 1 < high_i)
    {
        if (h > gamut_table.Load(int3(2, i, 0)))
        {
            low_i = i;
        }
        else
        {
            high_i = i;
        }
        i = midpoint(low_i, high_i);
    }
    lo = float3(gamut_table.Load(int3(0, high_i - 1, 0)),
        gamut_table.Load(int3(1, high_i - 1, 0)),
        gamut_table.Load(int3(2, high_i - 1, 0)));
    hi = float3(gamut_table.Load(int3(0, high_i, 0)),
        gamut_table.Load(int3(1, high_i, 0)),
        gamut_table.Load(int3(2, high_i, 0)));

    float t = (h - lo[2]) / (hi[2] - lo[2]);
    float cuspJ = lerp(lo[0], hi[0], t);
    float cuspM = lerp(lo[1], hi[1], t);

    float2 cuspJM = float2(cuspJ, cuspM);

    return cuspJM;
}


float3 tonemapAndCompress_fwd(float3 inputJMh,
                                ODTParams PARAMS)
{
    float3 outputJMh;

    float _linear = Hellwig_J_to_Y(inputJMh[0]) / referenceLuminance;

    TSParams TSPARAMS = init_TSParams(PARAMS.peakLuminance);
    float luminanceTS = tonescale_fwd(_linear, TSPARAMS);

    float tonemappedJ = Y_to_Hellwig_J(luminanceTS);

    float3 tonemappedJMh = float3(tonemappedJ, inputJMh[1], inputJMh[2]);

    outputJMh = tonemappedJMh;
    outputJMh[1] = chromaCompression(outputJMh,
                                     inputJMh[0],
                                     PARAMS,
                                     false);

    return outputJMh;
}

float3 tonemapAndCompress_inv(float3 JMh,
                                ODTParams PARAMS)
{
    float3 tonemappedJMh = JMh;

    float luminance = Hellwig_J_to_Y(JMh[0]);

    TSParams TSPARAMS = init_TSParams(PARAMS.peakLuminance);
    float linear_c = tonescale_inv(luminance / referenceLuminance,
                                 TSPARAMS);

    float untonemappedJ = Y_to_Hellwig_J(linear_c * referenceLuminance);
    float3 untonemappedColorJMh = float3(untonemappedJ, tonemappedJMh[1], tonemappedJMh[2]);

    // Chroma compression
    untonemappedColorJMh[1] = chromaCompression(tonemappedJMh,
                                                untonemappedColorJMh[0],
                                                PARAMS,
                                                true);

    return untonemappedColorJMh;
}

float3 JMh_to_output_XYZ(float3 JMh,
                           ODTParams PARAMS)
{
    float3 XYZluminance = Hellwig2022_JMh_to_XYZ(JMh,
                                                   PARAMS.XYZ_w_limit);

    float3 XYZ = (1. / referenceLuminance) * XYZluminance;

    return XYZ;
}

float3 XYZ_output_to_JMh(float3 XYZ,
                           ODTParams PARAMS)
{
    float3 XYZluminance = referenceLuminance * XYZ;

    float3 JMh = XYZ_to_Hellwig2022_JMh(XYZluminance,
                                          PARAMS.XYZ_w_limit);

    return JMh;
}

float hueDependentUpperHullGamma(float h,
                                 Texture2D<float> gamma_table)
{
    uint3 gammaSize;// = uint3(totalTableSize, 1, 1);
    gamma_table.GetDimensions(0, gammaSize.x, gammaSize.y, gammaSize.z);

    const int i_lo = hue_position_in_uniform_table(h, tableSize) + baseIndex;
    const int i_hi = next_position_in_table(i_lo, (int)gammaSize.x);

    const float base_hue = base_hue_for_position(i_lo - baseIndex, tableSize);

    const float t = wrap_to_360(h) - base_hue;

    return lerp(gamma_table.Load(int3(i_lo, 0, 0)), gamma_table.Load(int3(i_hi, 0, 0)), t);
}

float getFocusGain(float J,
                   float cuspJ,
                   float limitJmax)
{

    float thr = lerp(cuspJ, limitJmax, focusGainBlend);
    if (J > thr)
    {
        // Approximate inverse required above threshold
        float gain = (limitJmax - thr) / max(0.0001, limitJmax - J);
        return pow(log10(gain), 1. / focusAdjustGain) + 1.;
    }
    else
    {
        // Analytic inverse possible below cusp
        return 1.;
    }
}

float solve_J_intersect(float J,
                        float M,
                        float focusJ,
                        float maxJ,
                        float slope_gain)
{
    float a = M / (focusJ * slope_gain);
    float b = 0.0;
    float c = 0.0;
    float intersectJ = 0.0;

    if (J < focusJ)
    {
        b = 1.0 - M / slope_gain;
    }
    else
    {
        b = -(1.0 + M / slope_gain + maxJ * M / (focusJ * slope_gain));
    }

    if (J < focusJ)
    {
        c = -J;
    }
    else
    {
        c = maxJ * M / slope_gain + J;
    }

    float root = sqrt(b * b - 4.0 * a * c);

    if (J < focusJ)
    {
        intersectJ = 2.0 * c / (-b - root);
    }
    else
    {
        intersectJ = 2.0 * c / (-b + root);
    }

    return intersectJ;
}

float3 findGamutBoundaryIntersection(float3 JMh_s,
                                       float2 JM_cusp_in,
                                       float J_focus,
                                       float J_max,
                                       float slope_gain,
                                       float gamma_top,
                                       float gamma_bottom)
{
    float slope = 0.0;

    float s = max(0.000001, smoothCusps);
    float2 JM_cusp = JM_cusp_in;
    JM_cusp[1] = JM_cusp_in[1] * (1.0 + smoothM * s); // M

    float J_intersect_source = solve_J_intersect(JMh_s[0],
                                                 JMh_s[1],
                                                 J_focus,
                                                 J_max,
                                                 slope_gain);
    float J_intersect_cusp = solve_J_intersect(JM_cusp[0],
                                               JM_cusp[1],
                                               J_focus,
                                               J_max,
                                               slope_gain);

    if (J_intersect_source < J_focus)
    {
        slope = J_intersect_source * (J_intersect_source - J_focus) / (J_focus * slope_gain);
    }
    else
    {
        slope = (J_max - J_intersect_source) * (J_intersect_source - J_focus) / (J_focus * slope_gain);
    }

    float M_boundary_lower = J_intersect_cusp * pow(J_intersect_source / J_intersect_cusp, 1. / gamma_bottom) / (JM_cusp[0] / JM_cusp[1] - slope);

    float M_boundary_upper = JM_cusp[1] * (J_max - J_intersect_cusp) * pow((J_max - J_intersect_source) / (J_max - J_intersect_cusp), 1. / gamma_top) / (slope * JM_cusp[1] + J_max - JM_cusp[0]);

    float M_boundary = JM_cusp[1] * smin(M_boundary_lower / JM_cusp[1], M_boundary_upper / JM_cusp[1], s);

    float J_boundary = J_intersect_source + slope * M_boundary;

    float3 return_JMh = {J_boundary, M_boundary, J_intersect_source};

    return return_JMh;
}

float3 getReachBoundary(float J,
                          float M,
                          float h,
                          ODTParams PARAMS,
                          float2 JMcusp,
                          float focusJ)
{
    float limitJmax = PARAMS.limitJmax;
    float midJ = PARAMS.midJ;
    float model_gamma = PARAMS.model_gamma;
    float focusDist = PARAMS.focusDist;

    const float reachMaxM = reachMFromTable(h,  PARAMS.reach_table);

    float slope_gain = limitJmax * focusDist * getFocusGain(J, JMcusp[0], limitJmax);

    float intersectJ = solve_J_intersect(J, M, focusJ, limitJmax, slope_gain);
    float slope;
    if (intersectJ < focusJ)
    {
        slope = intersectJ * (intersectJ - focusJ) / (focusJ * slope_gain);
    }
    else
    {
        slope = (limitJmax - intersectJ) * (intersectJ - focusJ) / (focusJ * slope_gain);
    }
    float boundary = limitJmax * pow(intersectJ / limitJmax, model_gamma) * reachMaxM / (limitJmax - slope * reachMaxM);
    float3 result = {J, boundary, h};
    return result;
}

float3 compressGamut(float3 JMh,
                       ODTParams PARAMS,
                       float Jx,
                       bool _invert = false)
{
    float limitJmax = PARAMS.limitJmax;
    float midJ = PARAMS.midJ;
    float focusDist = PARAMS.focusDist;
    float model_gamma = PARAMS.model_gamma;

    float2 project_from = float2(JMh[0], JMh[1]);
    float2 JMcusp = cuspFromTable(JMh[2], PARAMS.gamut_table);

    if (JMh[1] < 0.0001 || JMh[0] > limitJmax)
    {
        float3 JMh_return = float3(JMh[0], 0.0, JMh[2]);
        return JMh_return;
    }

    // Calculate where the out of gamut color is projected to
    float focusJ = lerp(JMcusp[0], midJ, min(1., cuspMidBlend - (JMcusp[0] / limitJmax)));

    float slope_gain = limitJmax * focusDist * getFocusGain(Jx, JMcusp[0], limitJmax);

    // Find gamut intersection
    float gamma_top = hueDependentUpperHullGamma(JMh[2], PARAMS.gamma_table);
    float gamma_bottom = PARAMS.lowerHullGamma;

    float3 boundaryReturn = findGamutBoundaryIntersection(JMh,
                                                            JMcusp,
                                                            focusJ,
                                                            limitJmax,
                                                            slope_gain,
                                                            gamma_top,
                                                            gamma_bottom);

    float2 JMboundary = float2(boundaryReturn[0], boundaryReturn[1]);
    float2 project_to = float2(boundaryReturn[2], 0.0);
    float projectJ = boundaryReturn[2];

    // Calculate AP1 reach boundary
    float3 reachBoundary = getReachBoundary(JMboundary[0],
                                              JMboundary[1],
                                              JMh[2],
                                              PARAMS,
                                              JMcusp,
                                              focusJ);

    float difference = max(1.0001, reachBoundary[1] / JMboundary[1]);
    float threshold = max(compressionThreshold, 1. / difference);

    // Compress the out of gamut color along the projection line
    float v = project_from[1] / JMboundary[1];

    v = compressionFunction(v,
                            threshold,
                            difference,
                            _invert);

    float2 JMcompressed = project_to + v * (JMboundary - project_to);
    float3 return_JMh = float3(JMcompressed[0], JMcompressed[1], JMh[2]);

    return return_JMh;
}

float3 gamutMap_fwd(float3 JMh,
                      ODTParams PARAMS
                    )
{
    return compressGamut(JMh, PARAMS, JMh[0], false);
}

float3 gamutMap_inv(float3 JMh,
                      ODTParams PARAMS)
{
    float2 JMcusp = cuspFromTable(JMh[2], PARAMS.gamut_table);
    float Jx = JMh[0];

    // Analytic inverse below threshold
    if (Jx <= lerp(JMcusp[0], PARAMS.limitJmax, focusGainBlend))
        return compressGamut(JMh,
                             PARAMS,
                             Jx,
                             true);

    // Approximation above threshold
    Jx = compressGamut(JMh,
                       PARAMS,
                       Jx,
                       true)[0];
    return compressGamut(JMh,
                         PARAMS,
                         Jx,
                         true);
}

float3 outputTransform_fwd( float3 aces,
                            ODTParams PARAMS)
{
    float3 JMh = aces_to_JMh(aces, PARAMS.peakLuminance);

    float3 tonemappedJMh = tonemapAndCompress_fwd(JMh, PARAMS);

    float3 compressedJMh = gamutMap_fwd(tonemappedJMh, PARAMS);

    float3 XYZ = JMh_to_output_XYZ(compressedJMh,
                                      PARAMS);

    return XYZ;
}

float3 outputTransform_inv( float3 XYZ,
                            ODTParams PARAMS)
{
    float3 compressedJMh = XYZ_output_to_JMh(XYZ,
                                               PARAMS);

    float3 tonemappedJMh = gamutMap_inv(compressedJMh,
                                          PARAMS);

    float3 JMh = tonemapAndCompress_inv(tonemappedJMh,
                                          PARAMS);

    float3 aces = JMh_to_aces(JMh,
                                PARAMS.peakLuminance);

    return aces;
}