// Copyright Epic Games, Inc. All Rights Reserved.

// @todo: This should be moved to C++

// <ACEStransformID>urn:ampas:aces:transformId:v2.0:Lib.Academy.Tonescale.a2.v1</ACEStransformID>
// <ACESuserName>Tonescale Functions</ACESuserName>

// 
// Library File with functions used for pre-calculating the tonescale parameters and 
// applying the fwd/inv tonescale function
//

#pragma once

struct TSParams 
{
	float n;
	float n_r;
	float g;
	float t_1;
	float c_t; 
	float s_2;
	float u_2;
	float m_2;
};

// Tonescale pre-calculations
TSParams init_TSParams(float peakLuminance) {

    // Preset constants that set the desired behavior for the curve
    const float n = peakLuminance;
    
    static const float n_r = 100.0;    // normalized white in nits (what 1.0 should be)
    static const float g = 1.15;       // surround / contrast
    static const float c = 0.18;	    // anchor for 18% grey
    static const float c_d = 10.013;   // output luminance of 18% grey (in nits)
    static const float w_g = 0.14;     // change in grey between different peak luminance
    static const float t_1 = 0.04;     // shadow toe or flare/glare compensation
    static const float r_hit_min = 128.;	// scene-referred value "hitting the roof" for SDR (e.g. when n = 100 nits)
    static const float r_hit_max = 896.;   // scene-referred value "hitting the roof" for when n = 10000 nits

    // Calculate output constants
    const float r_hit = r_hit_min + (r_hit_max - r_hit_min) * (log(n/n_r)/log(10000./100.));
    const float m_0 = (n / n_r);
    const float m_1 = 0.5 * (m_0 + sqrt(m_0 * (m_0 + 4. * t_1)));
    const float u = pow((r_hit/m_1)/((r_hit/m_1)+1),g);
    const float m = m_1 / u;
    const float w_i = log(n/100.)/log(2.);
    const float c_t = c_d/n_r * (1. + w_i * w_g);
    const float g_ip = 0.5 * (c_t + sqrt(c_t * (c_t + 4. * t_1)));
    const float g_ipp2 = -(m_1 * pow((g_ip/m),(1./g))) / (pow( g_ip/m , 1./g)-1.);
    const float w_2 = c / g_ipp2;
    const float s_2 = w_2 * m_1;
    const float u_2 = pow( (r_hit/m_1)/((r_hit/m_1) + w_2), g);
    const float m_2 = m_1 / u_2;
    
    TSParams TonescaleParams = { 
    	n, 
    	n_r, 
    	g, 
    	t_1, 
    	c_t, 
    	s_2, 
    	u_2, 
    	m_2 
    };

    return TonescaleParams;
}


/* --- Tone scale math --- */
float tonescale_fwd( 
	float x, 				// scene-referred input (i.e. linear ACES2065-1)
	TSParams params 		// struct of type TSParams
) 	
{
	// forward MM tone scale
	float f = params.m_2 * pow( max(0.0, x) / (x + params.s_2), params.g);  	
	float h = max(0., f * f / (f + params.t_1));		 // forward flare 

	return h * params.n_r;	// output is luminance in cd/m^2
}

float tonescale_inv( 
	float Y,				// luminance in cd/m^2
	TSParams params 		// struct of type TSParams
)  
{
	float Z = max(0., min(params.n / (params.u_2 * params.n_r), Y));
	float h = (Z + sqrt(Z * (4. * params.t_1 + Z))) / 2.;
	float f = params.s_2 / (pow((params.m_2 / h), (1. / params.g)) - 1.);

	return f;				// output is scene-referred input
}
