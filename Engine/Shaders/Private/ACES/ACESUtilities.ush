// Copyright Epic Games, Inc. All Rights Reserved.

// <ACEStransformID>urn:ampas:aces:transformId:v2.0:Lib.Academy.Utilities.a2.v1</ACEStransformID>
// <ACESuserName>Utilities</ACESuserName>

//
// Generic functions that may be useful for writing CTL programs
//
#pragma once

#include "ACESCommon.ush"

#define UE_ACES_SAFE_FUNCTIONS 1

float radians_to_degrees( float radians )
{
	return radians * 180.0 / PI;
}

float degrees_to_radians( float degrees )
{
	return degrees / 180.0 * PI;
}

float copysign(float x, float y)
{
	return sign(y) * abs(x);
}

float3 copysign(float3 x, float3 y)
{
	return sign(y) * abs(x);
}

float smin(float a, float b, float s)
{
    float h = max(s - abs(a - b), 0.0) / s;
    return min(a, b) - h * h * h * s * (1.0 / 6.0);
}

#if UE_ACES_SAFE_FUNCTIONS

float spow(float base, float exponent)
{ 
	return (base < 0.0 && exponent != floor(exponent)) ? 0.0 : pow(base, exponent);
}

float3 spow(float3 base, float3 exponent)
{
	return float3(
        spow(base[0], exponent[0]), 
        spow(base[1], exponent[1]), 
		spow(base[2], exponent[2])
    );
}

float sdiv(float a, float b)
{
    return b == 0.0 ? 0.0 : a / b;
}

float satan2(float y, float x)
{
    return (x == 0.0 && y == 0.0) ? 0.0 : atan2(y, x);
}

#else // UE_ACES_SAFE_FUNCTIONS

#define spow pow

float sdiv(float a, float b)
{
    return a / b;
}

#define satan2 atan2

#endif // UE_SAFE_FUNCTIONS_SLOW
