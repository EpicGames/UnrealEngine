// Copyright Epic Games, Inc. All Rights Reserved.

/*
=============================================================================
	ACES: Academy Color Encoding System
	https://github.com/ampas/aces-dev/tree/v1.3

	License Terms for Academy Color Encoding System Components

	Academy Color Encoding System (ACES) software and tools are provided by the Academy under
	the following terms and conditions: A worldwide, royalty-free, non-exclusive right to copy, modify, create
	derivatives, and use, in source and binary forms, is hereby granted, subject to acceptance of this license.

	Copyright © 2015 Academy of Motion Picture Arts and Sciences (A.M.P.A.S.). Portions contributed by
	others as indicated. All rights reserved.

	Performance of any of the aforementioned acts indicates acceptance to be bound by the following
	terms and conditions:

	 *	Copies of source code, in whole or in part, must retain the above copyright
		notice, this list of conditions and the Disclaimer of Warranty.
	 *	Use in binary form must retain the above copyright notice, this list of
		conditions and the Disclaimer of Warranty in the documentation and/or other
		materials provided with the distribution.
	 *	Nothing in this license shall be deemed to grant any rights to trademarks,
		copyrights, patents, trade secrets or any other intellectual property of
		A.M.P.A.S. or any contributors, except as expressly stated herein.
	 *	Neither the name "A.M.P.A.S." nor the name of any other contributors to this
		software may be used to endorse or promote products derivative of or based on
		this software without express prior written permission of A.M.P.A.S. or the
		contributors, as appropriate.

	This license shall be construed pursuant to the laws of the State of California,
	and any disputes related thereto shall be subject to the jurisdiction of the courts therein.

	Disclaimer of Warranty: THIS SOFTWARE IS PROVIDED BY A.M.P.A.S. AND CONTRIBUTORS "AS
	IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
	IMPLIED WARRANTIES OF MERCHANTABILITY, FITNESS FOR A PARTICULAR PURPOSE, AND
	NON-INFRINGEMENT ARE DISCLAIMED. IN NO EVENT SHALL A.M.P.A.S., OR ANY
	CONTRIBUTORS OR DISTRIBUTORS, BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
	SPECIAL, EXEMPLARY, RESITUTIONARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT
	NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
	DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY THEORY
	OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT (INCLUDING
	NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS SOFTWARE,
	EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

	WITHOUT LIMITING THE GENERALITY OF THE FOREGOING, THE ACADEMY SPECIFICALLY
	DISCLAIMS ANY REPRESENTATIONS OR WARRANTIES WHATSOEVER RELATED TO PATENT OR
	OTHER INTELLECTUAL PROPERTY RIGHTS IN THE ACADEMY COLOR ENCODING SYSTEM, OR
	APPLICATIONS THEREOF, HELD BY PARTIES OTHER THAN A.M.P.A.S.,WHETHER DISCLOSED
	OR UNDISCLOSED.
=============================================================================
*/

/*
	we do matrix/matrix and matrix/vectors multiplications are following textbook order, like in our current implementation in ACES1.0.
	An example:
	in ACES:
		const float AP0_2_AP1_MAT[4][4] = mult_f44_f44( AP0_2_XYZ_MAT, XYZ_2_AP1_MAT);
		const float RGB_out[3] = mult_f3_f44( RGB_in, AP0_2_AP1_MAT);
	Equivalent math:
		R_out =  1.4514393161 * R_in + -0.2365107469 * G_in + -0.2149285693 * B_in;
		G_out = -0.0765537734 * R_in +  1.1762296998 * G_in + -0.0996759264 * B_in;
		B_out =  0.0083161484 * R_in + -0.0060324498 * G_in +  0.9977163014 * B_in;

	in HLSL:
	static const float3x3 AP0_2_AP1_MAT = //mul( XYZ_2_AP1_MAT, AP0_2_XYZ_MAT );
	{
		1.4514393161, -0.2365107469, -0.2149285693,
		-0.0765537734,  1.1762296998, -0.0996759264,
		0.0083161484, -0.0060324498,  0.9977163014,
	};
	float3 RGB_out = mul(AP0_2_AP1_MAT, RGB_in.xyz);
 */

 /////////////////////////////////////////////////////////////////////////////////////////

#pragma once

#include "ACESCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////

static const float HALF_MAX = 65504.0;
static const float HALF_POS_INF = 65535.0f;

static const float MIN_STOP_SDR = -6.5;
static const float MAX_STOP_SDR = 6.5;

static const float MIN_STOP_RRT = -15.;
static const float MAX_STOP_RRT = 18.;

static const float MIN_LUM_SDR = 0.02;
static const float MAX_LUM_SDR = 48.0;

static const float MIN_LUM_RRT = 0.0001;
static const float MAX_LUM_RRT = 10000.0;

float pow10(float x)
{
	return pow(10, x);
}

// ------- Red modifier functions
float cubic_basis_shaper
(
	float x,
	float w   // full base width of the shaper function (in degrees)
)
{
	float M[4][4] = { { -1. / 6,  3. / 6, -3. / 6,  1. / 6 },
					  {  3. / 6, -6. / 6,  3. / 6,  0. / 6 },
					  { -3. / 6,  0. / 6,  3. / 6,  0. / 6 },
					  {  1. / 6,  4. / 6,  1. / 6,  0. / 6 } };

	float knots[5] = { -0.5 * w, -0.25 * w, 0, 0.25 * w, 0.5 * w };

	float y = 0;
	if ((x > knots[0]) && (x < knots[4])) {
		float knot_coord = (x - knots[0]) * 4.0 / w;
		int j = knot_coord;
		float t = knot_coord - j;

		float monomials[4] = { t * t * t, t * t, t, 1.0 };

		// (if/else structure required for compatibility with CTL < v1.5.)
		if (j == 3) {
			y = monomials[0] * M[0][0] + monomials[1] * M[1][0] +
				monomials[2] * M[2][0] + monomials[3] * M[3][0];
		}
		else if (j == 2) {
			y = monomials[0] * M[0][1] + monomials[1] * M[1][1] +
				monomials[2] * M[2][1] + monomials[3] * M[3][1];
		}
		else if (j == 1) {
			y = monomials[0] * M[0][2] + monomials[1] * M[1][2] +
				monomials[2] * M[2][2] + monomials[3] * M[3][2];
		}
		else if (j == 0) {
			y = monomials[0] * M[0][3] + monomials[1] * M[1][3] +
				monomials[2] * M[2][3] + monomials[3] * M[3][3];
		}
		else {
			y = 0.0;
		}
	}

	return y * 1.5;
}

// note: this matrix is not transposed unlike other matrix operations because instead of doing v_out = mul(M, v_in), we do v_out = mul(v_in, M)
// Textbook monomial to basis-function conversion matrix.
static const float3x3 M1 =
{
	{  0.5, -1.0, 0.5 },
	{ -1.0,  1.0, 0.5 },
	{  0.5,  0.0, 0.0 }
};


/* ---- Functions to compress highlights ---- */
// allow for simulated white points without clipping

float roll_white_fwd(
	float inValue,      // color value to adjust (white scaled to around 1.0)
	float new_wht, // white adjustment (e.g. 0.9 for 10% darkening)
	float width    // adjusted width (e.g. 0.25 for top quarter of the tone scale)
)
{
	const float x0 = -1.0;
	const float x1 = x0 + width;
	const float y0 = -new_wht;
	const float y1 = x1;
	const float m1 = (x1 - x0);
	const float a = y0 - y1 + m1;
	const float b = 2 * (y1 - y0) - m1;
	const float c = y0;
	const float t = (-inValue - x0) / (x1 - x0);
	float outValue = 0.0;
	if (t < 0.0)
		outValue = -(t * b + c);
	else if (t > 1.0)
		outValue = inValue;
	else
		outValue = -((t * a + b) * t + c);
	return outValue;
}

float roll_white_rev(
	float inValue,      // color value to adjust (white scaled to around 1.0)
	float new_wht, // white adjustment (e.g. 0.9 for 10% darkening)
	float width    // adjusted width (e.g. 0.25 for top quarter of the tone scale)
)
{
	const float x0 = -1.0;
	const float x1 = x0 + width;
	const float y0 = -new_wht;
	const float y1 = x1;
	const float m1 = (x1 - x0);
	const float a = y0 - y1 + m1;
	const float b = 2. * (y1 - y0) - m1;
	float c = y0;
	float outValue = 0.0;
	if (-inValue < y0)
		outValue = -x0;
	else if (-inValue > y1)
		outValue = inValue;
	else {
		c = c + inValue;
		const float discrim = sqrt(b * b - 4. * a * c);
		const float t = (2. * c) / (-discrim - b);
		outValue = -((t * (x1 - x0)) + x0);
	}
	return outValue;
}

float3 limit_to_primaries
(
	float3 XYZ,
	FColorSpace LIMITING_PRI
)
{
	float3x3 XYZ_2_LIMITING_PRI_MAT = LIMITING_PRI.XYZtoRGB;
	float3x3 LIMITING_PRI_2_XYZ_MAT = LIMITING_PRI.RGBtoXYZ;

	// XYZ to limiting primaries
	float3 rgb = mul(XYZ_2_LIMITING_PRI_MAT, XYZ);

	// Clip any values outside the limiting primaries
	float3 limitedRgb = clamp(rgb, 0.0.xxx, 1.0.xxx);

	// Convert limited RGB to XYZ
	return mul(LIMITING_PRI_2_XYZ_MAT, limitedRgb);
}

///+ TODO: check if valid
float interpolate1D(const float2 table[2], float value)
{
	float t = saturate((value - table[0].x) / (table[1].x - table[0].x));
	return lerp(table[0].y, table[1].y, t);
}
///-

float lookup_ACESmin(float minLum)
{
	const float2 minTable[2] = { float2(log10(MIN_LUM_RRT), MIN_STOP_RRT),
								 float2(log10(MIN_LUM_SDR), MIN_STOP_SDR) };

	return 0.18 * pow(2., interpolate1D(minTable, log10(minLum)));
}

float lookup_ACESmax(float maxLum)
{
	const float2 maxTable[2] = { float2(log10(MAX_LUM_SDR), MAX_STOP_SDR),
								   float2(log10(MAX_LUM_RRT), MAX_STOP_RRT) };

	return 0.18 * pow(2., interpolate1D(maxTable, log10(maxLum)));
}

struct TsPoint
{
	float x;        // ACES
	float y;        // luminance
	float slope;    // 
};

struct TsParams
{
	TsPoint Min;
	TsPoint Mid;
	TsPoint Max;
	float coefsLow[6];
	float coefsHigh[6];
};

void init_coefsLow(
	TsPoint TsPointLow,
	TsPoint TsPointMid,
	out float coefsLow[5]
)
{

	float knotIncLow = (log10(TsPointMid.x) - log10(TsPointLow.x)) / 3.;
	// float halfKnotInc = (log10(TsPointMid.x) - log10(TsPointLow.x)) / 6.;

	// Determine two lowest coefficients (straddling minPt)
	coefsLow[0] = (TsPointLow.slope * (log10(TsPointLow.x) - 0.5 * knotIncLow)) + (log10(TsPointLow.y) - TsPointLow.slope * log10(TsPointLow.x));
	coefsLow[1] = (TsPointLow.slope * (log10(TsPointLow.x) + 0.5 * knotIncLow)) + (log10(TsPointLow.y) - TsPointLow.slope * log10(TsPointLow.x));
	// NOTE: if slope=0, then the above becomes just 
		// coefsLow[0] = log10(TsPointLow.y);
		// coefsLow[1] = log10(TsPointLow.y);
	// leaving it as a variable for now in case we decide we need non-zero slope extensions

	// Determine two highest coefficients (straddling midPt)
	coefsLow[3] = (TsPointMid.slope * (log10(TsPointMid.x) - 0.5 * knotIncLow)) + (log10(TsPointMid.y) - TsPointMid.slope * log10(TsPointMid.x));
	coefsLow[4] = (TsPointMid.slope * (log10(TsPointMid.x) + 0.5 * knotIncLow)) + (log10(TsPointMid.y) - TsPointMid.slope * log10(TsPointMid.x));

	// Middle coefficient (which defines the "sharpness of the bend") is linearly interpolated
	float2 bendsLow[2] = { float2(MIN_STOP_RRT, 0.18),
						   float2(MIN_STOP_SDR, 0.35) };
	float pctLow = interpolate1D(bendsLow, log2(TsPointLow.x / 0.18));
	coefsLow[2] = log10(TsPointLow.y) + pctLow * (log10(TsPointMid.y) - log10(TsPointLow.y));
}

void init_coefsHigh(
	TsPoint TsPointMid,
	TsPoint TsPointMax,
	out float coefsHigh[5]
)
{

	float knotIncHigh = (log10(TsPointMax.x) - log10(TsPointMid.x)) / 3.;
	// float halfKnotInc = (log10(TsPointMax.x) - log10(TsPointMid.x)) / 6.;

	// Determine two lowest coefficients (straddling midPt)
	coefsHigh[0] = (TsPointMid.slope * (log10(TsPointMid.x) - 0.5 * knotIncHigh)) + (log10(TsPointMid.y) - TsPointMid.slope * log10(TsPointMid.x));
	coefsHigh[1] = (TsPointMid.slope * (log10(TsPointMid.x) + 0.5 * knotIncHigh)) + (log10(TsPointMid.y) - TsPointMid.slope * log10(TsPointMid.x));

	// Determine two highest coefficients (straddling maxPt)
	coefsHigh[3] = (TsPointMax.slope * (log10(TsPointMax.x) - 0.5 * knotIncHigh)) + (log10(TsPointMax.y) - TsPointMax.slope * log10(TsPointMax.x));
	coefsHigh[4] = (TsPointMax.slope * (log10(TsPointMax.x) + 0.5 * knotIncHigh)) + (log10(TsPointMax.y) - TsPointMax.slope * log10(TsPointMax.x));
	// NOTE: if slope=0, then the above becomes just
		// coefsHigh[0] = log10(TsPointHigh.y);
		// coefsHigh[1] = log10(TsPointHigh.y);
	// leaving it as a variable for now in case we decide we need non-zero slope extensions

	// Middle coefficient (which defines the "sharpness of the bend") is linearly interpolated
	float2 bendsHigh[2] = { float2(MAX_STOP_SDR, 0.89) ,
							float2(MAX_STOP_RRT, 0.90) };
	float pctHigh = interpolate1D(bendsHigh, log2(TsPointMax.x / 0.18));
	coefsHigh[2] = log10(TsPointMid.y) + pctHigh * (log10(TsPointMax.y) - log10(TsPointMid.y));
}

float shift(float inValue, float expShift)
{
	return pow(2., (log2(inValue) - expShift));
}

TsParams init_TsParams(
	float minLum,
	float maxLum,
	float expShift = 0
)
{
	TsPoint MIN_PT = { lookup_ACESmin(minLum), minLum, 0.0 };
	TsPoint MID_PT = { 0.18, 4.8, 1.55 };
	TsPoint MAX_PT = { lookup_ACESmax(maxLum), maxLum, 0.0 };
	float cLow[5];
	init_coefsLow(MIN_PT, MID_PT, cLow);
	float cHigh[5];
	init_coefsHigh(MID_PT, MAX_PT, cHigh);
	MIN_PT.x = shift(lookup_ACESmin(minLum), expShift);
	MID_PT.x = shift(0.18, expShift);
	MAX_PT.x = shift(lookup_ACESmax(maxLum), expShift);

	TsParams P = {
		{MIN_PT.x, MIN_PT.y, MIN_PT.slope},
		{MID_PT.x, MID_PT.y, MID_PT.slope},
		{MAX_PT.x, MAX_PT.y, MAX_PT.slope},
		{cLow[0], cLow[1], cLow[2], cLow[3], cLow[4], cLow[4]},
		{cHigh[0], cHigh[1], cHigh[2], cHigh[3], cHigh[4], cHigh[4]}
	};

	return P;
}

float ssts
(
	const float x,
	const TsParams C
)
{
	const int N_KNOTS_LOW = 4;
	const int N_KNOTS_HIGH = 4;

	// Check for negatives or zero before taking the log. If negative or zero,
	// set to HALF_MIN
	float logx = log10(max(x, 1e-10));
	float logy;

	if (logx <= log10(C.Min.x)) {

		logy = logx * C.Min.slope + (log10(C.Min.y) - C.Min.slope * log10(C.Min.x));

	}
	else if ((logx > log10(C.Min.x)) && (logx < log10(C.Mid.x))) {

		float knot_coord = (N_KNOTS_LOW - 1) * (logx - log10(C.Min.x)) / (log10(C.Mid.x) - log10(C.Min.x));
		int j = knot_coord;
		float t = knot_coord - j;

		float3 cf = { C.coefsLow[j], C.coefsLow[j + 1], C.coefsLow[j + 2] };

		float3 monomials = { t * t, t, 1.0 };
		logy = dot(monomials, mul(cf, M1));

	}
	else if ((logx >= log10(C.Mid.x)) && (logx < log10(C.Max.x))) {

		float knot_coord = (N_KNOTS_HIGH - 1) * (logx - log10(C.Mid.x)) / (log10(C.Max.x) - log10(C.Mid.x));
		int j = knot_coord;
		float t = knot_coord - j;

		float3 cf = { C.coefsHigh[j], C.coefsHigh[j + 1], C.coefsHigh[j + 2] };

		float3 monomials = { t * t, t, 1.0 };
		logy = dot(monomials, mul(cf, M1));

	}
	else { //if ( logIn >= log10(C.Max.x) ) { 

		logy = logx * C.Max.slope + (log10(C.Max.y) - C.Max.slope * log10(C.Max.x));

	}

	return pow10(logy);

}

float inv_ssts
(
	const float y,
	const TsParams C
)
{
	const int N_KNOTS_LOW = 4;
	const int N_KNOTS_HIGH = 4;

	const float KNOT_INC_LOW = (log10(C.Mid.x) - log10(C.Min.x)) / (N_KNOTS_LOW - 1.);
	const float KNOT_INC_HIGH = (log10(C.Max.x) - log10(C.Mid.x)) / (N_KNOTS_HIGH - 1.);

	// KNOT_Y is luminance of the spline at each knot
	float KNOT_Y_LOW[N_KNOTS_LOW];
	///+warning: redefinition of 'i'
	{
		for (int i = 0; i < N_KNOTS_LOW; i = i + 1) {
			KNOT_Y_LOW[i] = (C.coefsLow[i] + C.coefsLow[i + 1]) / 2.;
		};
	}
	///-

	float KNOT_Y_HIGH[N_KNOTS_HIGH];
	///+ warning: redefinition of 'i'
	{
		for (int i = 0; i < N_KNOTS_HIGH; i = i + 1) {
			KNOT_Y_HIGH[i] = (C.coefsHigh[i] + C.coefsHigh[i + 1]) / 2.;
		};
	}
	///-
	float logy = log10(max(y, 1e-10));

	float logx;
	if (logy <= log10(C.Min.y)) {

		logx = log10(C.Min.x);

	}
	else if ((logy > log10(C.Min.y)) && (logy <= log10(C.Mid.y))) {

		int j;
		float3 cf;
		if (logy > KNOT_Y_LOW[0] && logy <= KNOT_Y_LOW[1]) {
			cf[0] = C.coefsLow[0];  cf[1] = C.coefsLow[1];  cf[2] = C.coefsLow[2];  j = 0;
		}
		else if (logy > KNOT_Y_LOW[1] && logy <= KNOT_Y_LOW[2]) {
			cf[0] = C.coefsLow[1];  cf[1] = C.coefsLow[2];  cf[2] = C.coefsLow[3];  j = 1;
		}
		else if (logy > KNOT_Y_LOW[2] && logy <= KNOT_Y_LOW[3]) {
			cf[0] = C.coefsLow[2];  cf[1] = C.coefsLow[3];  cf[2] = C.coefsLow[4];  j = 2;
		}

		const float3 tmp = mul(cf, M1);

		float a = tmp[0];
		float b = tmp[1];
		float c = tmp[2];
		c = c - logy;

		const float d = sqrt(b * b - 4. * a * c);

		const float t = (2. * c) / (-d - b);

		logx = log10(C.Min.x) + (t + j) * KNOT_INC_LOW;

	}
	else if ((logy > log10(C.Mid.y)) && (logy < log10(C.Max.y))) {

		int j;
		float3 cf;
		if (logy >= KNOT_Y_HIGH[0] && logy <= KNOT_Y_HIGH[1]) {
			cf[0] = C.coefsHigh[0];  cf[1] = C.coefsHigh[1];  cf[2] = C.coefsHigh[2];  j = 0;
		}
		else if (logy > KNOT_Y_HIGH[1] && logy <= KNOT_Y_HIGH[2]) {
			cf[0] = C.coefsHigh[1];  cf[1] = C.coefsHigh[2];  cf[2] = C.coefsHigh[3];  j = 1;
		}
		else if (logy > KNOT_Y_HIGH[2] && logy <= KNOT_Y_HIGH[3]) {
			cf[0] = C.coefsHigh[2];  cf[1] = C.coefsHigh[3];  cf[2] = C.coefsHigh[4];  j = 2;
		}

		const float3 tmp = mul(cf, M1);

		float a = tmp[0];
		float b = tmp[1];
		float c = tmp[2];
		c = c - logy;

		const float d = sqrt(b * b - 4. * a * c);

		const float t = (2. * c) / (-d - b);

		logx = log10(C.Mid.x) + (t + j) * KNOT_INC_HIGH;

	}
	else { //if ( logy >= log10(C.Max.y) ) {

		logx = log10(C.Max.x);

	}

	return pow10(logx);

}

float3 ssts_f3
(
	const float3 x,
	const TsParams C
)
{
	float3 outValue;
	outValue[0] = ssts(x[0], C);
	outValue[1] = ssts(x[1], C);
	outValue[2] = ssts(x[2], C);

	return outValue;
}

float3 inv_ssts_f3
(
	const float3 x,
	const TsParams C
)
{
	float3 outValue;
	outValue[0] = inv_ssts(x[0], C);
	outValue[1] = inv_ssts(x[1], C);
	outValue[2] = inv_ssts(x[2], C);

	return outValue;
}

float moncurve_f(float x, float gamma, float offs)
{
	// Forward monitor curve
	float y;
	const float fs = ((gamma - 1.0) / offs) * pow(offs * gamma / ((gamma - 1.0) * (1.0 + offs)), gamma);
	const float xb = offs / (gamma - 1.0);
	if (x >= xb)
		y = pow((x + offs) / (1.0 + offs), gamma);
	else
		y = x * fs;
	return y;
}

float moncurve_r(float y, float gamma, float offs)
{
	// Reverse monitor curve
	float x;
	const float yb = pow(offs * gamma / ((gamma - 1.0) * (1.0 + offs)), gamma);
	const float rs = pow((gamma - 1.0) / offs, gamma - 1.0) * pow((1.0 + offs) / gamma, gamma);
	if (y >= yb)
		x = (1.0 + offs) * pow(y, 1.0 / gamma) - offs;
	else
		x = y * rs;
	return x;
}

float3 moncurve_f_f3(float3 x, float gamma, float offs)
{
	float3 y;
	y[0] = moncurve_f(x[0], gamma, offs);
	y[1] = moncurve_f(x[1], gamma, offs);
	y[2] = moncurve_f(x[2], gamma, offs);
	return y;
}

float3 moncurve_r_f3(float3 y, float gamma, float offs)
{
	float3 x;
	x[0] = moncurve_r(y[0], gamma, offs);
	x[1] = moncurve_r(y[1], gamma, offs);
	x[2] = moncurve_r(y[2], gamma, offs);
	return x;
}

float bt1886_f(float V, float gamma, float Lw, float Lb)
{
	// The reference EOTF specified in Rec. ITU-R BT.1886
	// L = a(max[(V+b),0])^g
	float a = pow(pow(Lw, 1. / gamma) - pow(Lb, 1. / gamma), gamma);
	float b = pow(Lb, 1. / gamma) / (pow(Lw, 1. / gamma) - pow(Lb, 1. / gamma));
	float L = a * pow(max(V + b, 0.), gamma);
	return L;
}

float bt1886_r(float L, float gamma, float Lw, float Lb)
{
	// The reference EOTF specified in Rec. ITU-R BT.1886
	// L = a(max[(V+b),0])^g
	float a = pow(pow(Lw, 1. / gamma) - pow(Lb, 1. / gamma), gamma);
	float b = pow(Lb, 1. / gamma) / (pow(Lw, 1. / gamma) - pow(Lb, 1. / gamma));
	float V = pow(max(L / a, 0.), 1. / gamma) - b;
	return V;
}

float3 bt1886_f_f3(float3 V, float gamma, float Lw, float Lb)
{
	float3 L;
	L[0] = bt1886_f(V[0], gamma, Lw, Lb);
	L[1] = bt1886_f(V[1], gamma, Lw, Lb);
	L[2] = bt1886_f(V[2], gamma, Lw, Lb);
	return L;
}

float3 bt1886_r_f3(float3 L, float gamma, float Lw, float Lb)
{
	float3 V;
	V[0] = bt1886_r(L[0], gamma, Lw, Lb);
	V[1] = bt1886_r(L[1], gamma, Lw, Lb);
	V[2] = bt1886_r(L[2], gamma, Lw, Lb);
	return V;
}


// SMPTE Range vs Full Range scaling formulas
float smpteRange_to_fullRange(float inValue)
{
	const float REFBLACK = (64. / 1023.);
	const float REFWHITE = (940. / 1023.);

	return ((inValue - REFBLACK) / (REFWHITE - REFBLACK));
}

float fullRange_to_smpteRange(float inValue)
{
	const float REFBLACK = (64. / 1023.);
	const float REFWHITE = (940. / 1023.);

	return (inValue * (REFWHITE - REFBLACK) + REFBLACK);
}

float3 smpteRange_to_fullRange_f3(float3 rgbIn)
{
	float3 rgbOut;
	rgbOut[0] = smpteRange_to_fullRange(rgbIn[0]);
	rgbOut[1] = smpteRange_to_fullRange(rgbIn[1]);
	rgbOut[2] = smpteRange_to_fullRange(rgbIn[2]);

	return rgbOut;
}

float3 fullRange_to_smpteRange_f3(float3 rgbIn)
{
	float3 rgbOut;
	rgbOut[0] = fullRange_to_smpteRange(rgbIn[0]);
	rgbOut[1] = fullRange_to_smpteRange(rgbIn[1]);
	rgbOut[2] = fullRange_to_smpteRange(rgbIn[2]);

	return rgbOut;
}

// Base functions from SMPTE ST 2084-2014

// Constants from SMPTE ST 2084-2014
static const float pq_m1 = 0.1593017578125; // ( 2610.0 / 4096.0 ) / 4.0;
static const float pq_m2 = 78.84375; // ( 2523.0 / 4096.0 ) * 128.0;
static const float pq_c1 = 0.8359375; // 3424.0 / 4096.0 or pq_c3 - pq_c2 + 1.0;
static const float pq_c2 = 18.8515625; // ( 2413.0 / 4096.0 ) * 32.0;
static const float pq_c3 = 18.6875; // ( 2392.0 / 4096.0 ) * 32.0;

static const float pq_C = 10000.0;

// Converts from the non-linear perceptually quantized space to linear cd/m^2
// Note that this is in float, and assumes normalization from 0 - 1
// (0 - pq_C for linear) and does not handle the integer coding in the Annex 
// sections of SMPTE ST 2084-2014
float ST2084_2_Y(float N)
{
	// Note that this does NOT handle any of the signal range
	// considerations from 2084 - this assumes full range (0 - 1)
	float Np = pow(N, 1.0 / pq_m2);
	float L = Np - pq_c1;
	if (L < 0.0)
		L = 0.0;
	L = L / (pq_c2 - pq_c3 * Np);
	L = pow(L, 1.0 / pq_m1);
	return L * pq_C; // returns cd/m^2
}

// Converts from linear cd/m^2 to the non-linear perceptually quantized space
// Note that this is in float, and assumes normalization from 0 - 1
// (0 - pq_C for linear) and does not handle the integer coding in the Annex 
// sections of SMPTE ST 2084-2014
float Y_2_ST2084(float C)
//pq_r
{
	// Note that this does NOT handle any of the signal range
	// considerations from 2084 - this returns full range (0 - 1)
	float L = C / pq_C;
	float Lm = pow(L, pq_m1);
	float N = (pq_c1 + pq_c2 * Lm) / (1.0 + pq_c3 * Lm);
	N = pow(N, pq_m2);
	return N;
}

float3 Y_2_ST2084_f3(float3 inValue)
{
	// converts from linear cd/m^2 to PQ code values

	float3 outValue;
	outValue[0] = Y_2_ST2084(inValue[0]);
	outValue[1] = Y_2_ST2084(inValue[1]);
	outValue[2] = Y_2_ST2084(inValue[2]);

	return outValue;
}

float3 ST2084_2_Y_f3(float3 inValue)
{
	// converts from PQ code values to linear cd/m^2

	float3 outValue;
	outValue[0] = ST2084_2_Y(inValue[0]);
	outValue[1] = ST2084_2_Y(inValue[1]);
	outValue[2] = ST2084_2_Y(inValue[2]);

	return outValue;
}

// Conversion of PQ signal to HLG, as detailed in Section 7 of ITU-R BT.2390-0
float3 ST2084_2_HLG_1000nits_f3(float3 PQ)
{
	// ST.2084 EOTF (non-linear PQ to display light)
	float3 displayLinear = ST2084_2_Y_f3(PQ);

	// HLG Inverse EOTF (i.e. HLG inverse OOTF followed by the HLG OETF)
	// HLG Inverse OOTF (display linear to scene linear)
	float Y_d = 0.2627 * displayLinear[0] + 0.6780 * displayLinear[1] + 0.0593 * displayLinear[2];
	const float L_w = 1000.;
	const float L_b = 0.;
	const float alpha = (L_w - L_b);
	const float beta = L_b;
	const float gamma = 1.2;

	float3 sceneLinear;
	if (Y_d == 0.) {
		/* This case is to protect against pow(0,-N)=Inf error. The ITU document
		does not offer a recommendation for this corner case. There may be a
		better way to handle this, but for now, this works.
		*/
		sceneLinear[0] = 0.;
		sceneLinear[1] = 0.;
		sceneLinear[2] = 0.;
	}
	else {
		sceneLinear[0] = pow((Y_d - beta) / alpha, (1. - gamma) / gamma) * ((displayLinear[0] - beta) / alpha);
		sceneLinear[1] = pow((Y_d - beta) / alpha, (1. - gamma) / gamma) * ((displayLinear[1] - beta) / alpha);
		sceneLinear[2] = pow((Y_d - beta) / alpha, (1. - gamma) / gamma) * ((displayLinear[2] - beta) / alpha);
	}

	// HLG OETF (scene linear to non-linear signal value)
	const float a = 0.17883277;
	const float b = 0.28466892; // 1.-4.*a;
	const float c = 0.55991073; // 0.5-a*log(4.*a);

	float3 HLG;
	if (sceneLinear[0] <= 1. / 12) {
		HLG[0] = sqrt(3. * sceneLinear[0]);
	}
	else {
		HLG[0] = a * log(12. * sceneLinear[0] - b) + c;
	}
	if (sceneLinear[1] <= 1. / 12) {
		HLG[1] = sqrt(3. * sceneLinear[1]);
	}
	else {
		HLG[1] = a * log(12. * sceneLinear[1] - b) + c;
	}
	if (sceneLinear[2] <= 1. / 12) {
		HLG[2] = sqrt(3. * sceneLinear[2]);
	}
	else {
		HLG[2] = a * log(12. * sceneLinear[2] - b) + c;
	}

	return HLG;
}


// Conversion of HLG to PQ signal, as detailed in Section 7 of ITU-R BT.2390-0
float3 HLG_2_ST2084_1000nits_f3(float3 HLG)
{
	const float a = 0.17883277;
	const float b = 0.28466892; // 1.-4.*a;
	const float c = 0.55991073; // 0.5-a*log(4.*a);

	const float L_w = 1000.;
	const float L_b = 0.;
	const float alpha = (L_w - L_b);
	const float beta = L_b;
	const float gamma = 1.2;

	// HLG EOTF (non-linear signal value to display linear)
	// HLG to scene-linear
	float3 sceneLinear;
	if (HLG[0] >= 0. && HLG[0] <= 0.5) {
		sceneLinear[0] = pow(HLG[0], 2.) / 3.;
	}
	else {
		sceneLinear[0] = (exp((HLG[0] - c) / a) + b) / 12.;
	}
	if (HLG[1] >= 0. && HLG[1] <= 0.5) {
		sceneLinear[1] = pow(HLG[1], 2.) / 3.;
	}
	else {
		sceneLinear[1] = (exp((HLG[1] - c) / a) + b) / 12.;
	}
	if (HLG[2] >= 0. && HLG[2] <= 0.5) {
		sceneLinear[2] = pow(HLG[2], 2.) / 3.;
	}
	else {
		sceneLinear[2] = (exp((HLG[2] - c) / a) + b) / 12.;
	}

	float Y_s = 0.2627 * sceneLinear[0] + 0.6780 * sceneLinear[1] + 0.0593 * sceneLinear[2];

	// Scene-linear to display-linear
	float3 displayLinear;
	displayLinear[0] = alpha * pow(Y_s, gamma - 1.) * sceneLinear[0] + beta;
	displayLinear[1] = alpha * pow(Y_s, gamma - 1.) * sceneLinear[1] + beta;
	displayLinear[2] = alpha * pow(Y_s, gamma - 1.) * sceneLinear[2] + beta;

	// ST.2084 Inverse EOTF
	float3 PQ = Y_2_ST2084_f3(displayLinear);

	return PQ;
}

// Desaturation contants
static const float RRT_SAT_FACTOR = 0.96;
static const float ONE_MINUS_RRT_SAT_FACTOR = 0.04;
static const float3x3 RRT_SAT_MAT =
{
	//    {ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.x + RRT_SAT_FACTOR, ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.y,                  ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.z},
	//    {ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.x,                  ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.y + RRT_SAT_FACTOR, ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.z},
	//    {ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.x,                  ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.y,                  ONE_MINUS_RRT_SAT_FACTOR * AP1_RGB2Y.z + RRT_SAT_FACTOR},

		{0.970889148672, 0.02696327063, 0.0021475807},
		{0.010889148672, 0.98696327063, 0.0021475807},
		{0.010889148672, 0.02696327063, 0.9621475807}
};

static const float3x3 RRT_SAT_MAT_INV =
{
	{  1.03032386   , -0.0280867405 , -0.00223706313 },
	{ -0.0113428626 , 1.01357996    , -0.00223706337 },
	{ -0.0113428626 , -0.0280867405 , 1.03942955 }
};


// "Glow" module constants
static const float RRT_GLOW_GAIN = 0.05;
static const float RRT_GLOW_MID = 0.08;

// Red modifier constants
static const float RRT_RED_SCALE = 0.82;
static const float RRT_RED_PIVOT = 0.03;
static const float RRT_RED_HUE = 0.;
static const float RRT_RED_WIDTH = 135.;


float3 rrt_sweeteners(float3 inValue)
{
	float3 aces = inValue;

	// --- Glow module --- //
	float saturation = rgb_2_saturation(aces);
	float ycIn = rgb_2_yc(aces);
	float s = sigmoid_shaper((saturation - 0.4) / 0.2);
	float addedGlow = 1 + glow_fwd(ycIn, RRT_GLOW_GAIN * s, RRT_GLOW_MID);
	aces *= addedGlow;

	// --- Red modifier --- //
	float hue = rgb_2_hue(aces);
	float centeredHue = center_hue(hue, RRT_RED_HUE);
	float hueWeight = cubic_basis_shaper(centeredHue, RRT_RED_WIDTH);

	aces.r += hueWeight * saturation * (RRT_RED_PIVOT - aces.r) * (1. - RRT_RED_SCALE);

	// --- ACES to RGB rendering space --- //
	aces = clamp(aces, 0, HALF_POS_INF);
	float3 rgbPre = mul(AP0_2_AP1_MAT, aces);
	rgbPre = clamp(rgbPre, 0, HALF_MAX);

	// --- Global desaturation --- //
	//rgbPre = mul(RRT_SAT_MAT, rgbPre);
	rgbPre = lerp(dot(rgbPre, AP1_RGB2Y).xxx, rgbPre, RRT_SAT_FACTOR);
	return rgbPre;
}

float3 inv_rrt_sweeteners(float3 inValue)
{
	float3 rgbPost = inValue;

	// --- Global desaturation --- //
	rgbPost = mul(RRT_SAT_MAT_INV, rgbPost);

	rgbPost = clamp(rgbPost, 0.0.xxx, HALF_MAX.xxx);

	// --- RGB rendering space to ACES --- //
	float3 aces = mul(AP1_2_AP0_MAT, rgbPost);

	aces = clamp(aces, 0.0.xxx, HALF_MAX.xxx);

	// --- Red modifier --- //
	float hue = rgb_2_hue(aces);
	float centeredHue = center_hue(hue, RRT_RED_HUE);
	float hueWeight = cubic_basis_shaper(centeredHue, RRT_RED_WIDTH);

	float minChan;
	if (centeredHue < 0) { // min_f3(aces) = aces[1] (i.e. magenta-red)
		minChan = aces[1];
	}
	else { // min_f3(aces) = aces[2] (i.e. yellow-red)
		minChan = aces[2];
	}

	float a = hueWeight * (1. - RRT_RED_SCALE) - 1.;
	float b = aces[0] - hueWeight * (RRT_RED_PIVOT + minChan) * (1. - RRT_RED_SCALE);
	float c = hueWeight * RRT_RED_PIVOT * minChan * (1. - RRT_RED_SCALE);

	aces[0] = (-b - sqrt(b * b - 4. * a * c)) / (2. * a);

	// --- Glow module --- //
	float saturation = rgb_2_saturation(aces);
	float ycOut = rgb_2_yc(aces);
	float s = sigmoid_shaper((saturation - 0.4) / 0.2);
	float reducedGlow = 1. + glow_inv(ycOut, RRT_GLOW_GAIN * s, RRT_GLOW_MID);

	aces *= reducedGlow;
	return aces;
}

float Y_2_linCV(float Y, float Ymax, float Ymin)
{
	return (Y - Ymin) / (Ymax - Ymin);
}

float linCV_2_Y(float linCV, float Ymax, float Ymin)
{
	return linCV * (Ymax - Ymin) + Ymin;
}

float3 Y_2_linCV_f3(float3 Y, float Ymax, float Ymin)
{
	float3 linCV;
	linCV[0] = Y_2_linCV(Y[0], Ymax, Ymin);
	linCV[1] = Y_2_linCV(Y[1], Ymax, Ymin);
	linCV[2] = Y_2_linCV(Y[2], Ymax, Ymin);
	return linCV;
}

float3 linCV_2_Y_f3(float3 linCV, float Ymax, float Ymin)
{
	float3 Y;
	Y[0] = linCV_2_Y(linCV[0], Ymax, Ymin);
	Y[1] = linCV_2_Y(linCV[1], Ymax, Ymin);
	Y[2] = linCV_2_Y(linCV[2], Ymax, Ymin);
	return Y;
}

// Gamma compensation factor
static const float DIM_SURROUND_GAMMA = 0.9811;

float3 darkSurround_to_dimSurround(float3 linearCV)
{
	float3 XYZ = mul(AP1_2_XYZ_MAT, linearCV);

	float3 xyY = XYZ_2_xyY(XYZ);
	xyY[2] = clamp(xyY[2], 0., HALF_POS_INF);
	xyY[2] = pow(xyY[2], DIM_SURROUND_GAMMA);
	XYZ = xyY_2_XYZ(xyY);

	return mul(XYZ_2_AP1_MAT, XYZ);
}

float3 dimSurround_to_darkSurround(float3 linearCV)
{
	float3 XYZ = mul(AP1_2_XYZ_MAT, linearCV);

	float3 xyY = XYZ_2_xyY(XYZ);
	xyY[2] = clamp(xyY[2], 0., HALF_POS_INF);
	xyY[2] = pow(xyY[2], 1. / DIM_SURROUND_GAMMA);
	XYZ = xyY_2_XYZ(xyY);

	return mul(XYZ_2_AP1_MAT, XYZ);
}

float3 dark_to_dim(float3 XYZ)
{
	float3 xyY = XYZ_2_xyY(XYZ);
	xyY[2] = clamp(xyY[2], 0., HALF_POS_INF);
	xyY[2] = pow(xyY[2], DIM_SURROUND_GAMMA);
	return xyY_2_XYZ(xyY);
}

float3 dim_to_dark(float3 XYZ)
{
	float3 xyY = XYZ_2_xyY(XYZ);
	xyY[2] = clamp(xyY[2], 0., HALF_POS_INF);
	xyY[2] = pow(xyY[2], 1. / DIM_SURROUND_GAMMA);
	return xyY_2_XYZ(xyY);
}

float3 outputTransform
(
	float3 inValue,
	TsParams PARAMS,
	FColorSpace DISPLAY_PRI,
	FColorSpace LIMITING_PRI,
	int EOTF,
	int SURROUND,
	bool STRETCH_BLACK = true,
	bool D60_SIM = false,
	bool LEGAL_RANGE = false
)
{
	float3x3 XYZ_2_DISPLAY_PRI_MAT = DISPLAY_PRI.XYZtoRGB;

	/*
		NOTE: This is a bit of a hack - probably a more direct way to do this.
		TODO: Fix in future version
	*/

	float Y_MIN = PARAMS.Min.y;
	float Y_MAX = PARAMS.Max.y;

	// RRT sweeteners
	float3 rgbPre = rrt_sweeteners(inValue);

	// Apply the tonescale independently in rendering-space RGB
	float3 rgbPost = ssts_f3(rgbPre, PARAMS);

	// At this point data encoded AP1, scaled absolute luminance (cd/m^2)

	/*  Scale absolute luminance to linear code value  */
	float3 linearCV = Y_2_linCV_f3(rgbPost, Y_MAX, Y_MIN);

	// Rendering primaries to XYZ
	float3 XYZ = mul(AP1_2_XYZ_MAT, linearCV);

	// Apply gamma adjustment to compensate for dim surround
	/*
		NOTE: This is more or less a placeholder block and is largely inactive
		in its current form. This section currently only applies for SDR, and
		even then, only in very specific cases.
		In the future it is fully intended for this module to be updated to
		support surround compensation regardless of luminance dynamic range. */
		/*
			TOD0: Come up with new surround compensation algorithm, applicable
			across all dynamic ranges and supporting dark/dim/normal surround.
		*/
	if (SURROUND == 0) { // Dark surround
		/*
		Current tone scale is designed for dark surround environment so no
		adjustment is necessary.
		*/
	}
	else if (SURROUND == 1) { // Dim surround
	 // INACTIVE for HDR and crudely implemented for SDR (see comment below)        
		if ((EOTF == 1) || (EOTF == 2) || (EOTF == 3)) {
			/*
			This uses a crude logical assumption that if the EOTF is BT.1886,
			sRGB, or gamma 2.6 that the data is SDR and so the SDR gamma
			compensation factor from v1.0 will apply.
			*/
			XYZ = dark_to_dim(XYZ); /*
			This uses a local dark_to_dim function that is designed to take in
			XYZ and return XYZ rather than AP1 as is currently in the functions
			in 'ACESlib.ODT_Common.ctl' */
		}
	}
	else if (SURROUND == 2) { // Normal surround
	 // INACTIVE - this does NOTHING
	}

	// Gamut limit to limiting primaries
	// NOTE: Would be nice to just say
	//    if (LIMITING_PRI != DISPLAY_PRI)
	// but you can't because Chromaticities do not work with bool comparison operator
	// For now, limit no matter what.
	XYZ = limit_to_primaries(XYZ, LIMITING_PRI);

	// Apply CAT from ACES white point to assumed observer adapted white point
	// TODO: Needs to expand from just supporting D60 sim to allow for any
	// observer adapted white point.
	if (D60_SIM == false) {
		if (DISPLAY_PRI.White != WhiteStandard_D60) {
			XYZ = mul(D60_2_D65_CAT, XYZ);
		}
	}

	// CIE XYZ to display encoding primaries
	linearCV = mul(XYZ_2_DISPLAY_PRI_MAT, XYZ);

	// Scale to avoid clipping when device calibration is different from D60. 
	// To simulate D60, unequal code values are sent to the display.
	// TODO: Needs to expand from just supporting D60 sim to allow for any
	// observer adapted white point.
	if (D60_SIM == true) {
		/* TODO: The scale requires calling itself. Scale is same no matter the luminance.
		   Currently precalculated for D65, DCI. If DCI, roll_white_fwd is used also.
		   This needs a more complex algorithm to handle all cases.
		*/
		float SCALE = 1.0;
		if (DISPLAY_PRI.White == WhiteStandard_D65) { // D65
			SCALE = 0.96362;
		}
		else if (DISPLAY_PRI.White == WhiteStandard_DCI) { // DCI
			linearCV[0] = roll_white_fwd(linearCV[0], 0.918, 0.5);
			linearCV[1] = roll_white_fwd(linearCV[1], 0.918, 0.5);
			linearCV[2] = roll_white_fwd(linearCV[2], 0.918, 0.5);
			SCALE = 0.96;
		}
		linearCV *= SCALE;
	}


	// Clip values < 0 (i.e. projecting outside the display primaries)
	// NOTE: P3 red and values close to it fall outside of Rec.2020 green-red 
	// boundary
	linearCV = clamp(linearCV, 0.0.xxx, HALF_POS_INF.xxx);

	// EOTF
	// 0: ST-2084 (PQ)
	// 1: BT.1886 (Rec.709/2020 settings)
	// 2: sRGB (mon_curve w/ presets)
	//    moncurve_r with gamma of 2.4 and offset of 0.055 matches the EOTF found in IEC 61966-2-1:1999 (sRGB)
	// 3: gamma 2.6
	// 4: linear (no EOTF)
	// 5: HLG
	float3 outputCV;
	if (EOTF == 0) {  // ST-2084 (PQ)
		// NOTE: This is a kludgy way of ensuring a PQ code value of 0. Ideally,
		// luminance would map directly to code value, but colorists don't like
		// that. Might just need the tonescale to go darker so that darkest
		// values through the tone scale quantize to code value of 0.
		if (STRETCH_BLACK == true) {
			outputCV = Y_2_ST2084_f3(clamp(linCV_2_Y_f3(linearCV, Y_MAX, 0.0), 0.0, HALF_POS_INF));
		}
		else {
			outputCV = Y_2_ST2084_f3(linCV_2_Y_f3(linearCV, Y_MAX, Y_MIN));
		}
	}
	else if (EOTF == 1) { // BT.1886 (Rec.709/2020 settings)
		outputCV = bt1886_r_f3(linearCV, 2.4, 1.0, 0.0);
	}
	else if (EOTF == 2) { // sRGB (mon_curve w/ presets)
		outputCV = moncurve_r_f3(linearCV, 2.4, 0.055);
	}
	else if (EOTF == 3) { // gamma 2.6
		outputCV = pow(linearCV, (1. / 2.6).xxx);
	}
	else if (EOTF == 4) { // linear
	 ///+ have linear behave the same way at ST-2084 regarding STRETCH_BLACK option
		if (STRETCH_BLACK == true) {
			outputCV = clamp(linCV_2_Y_f3(linearCV, Y_MAX, 0.0), 0.0, HALF_POS_INF);
		}
		else {
			outputCV = linCV_2_Y_f3(linearCV, Y_MAX, Y_MIN);
		}
	}
	else if (EOTF == 5) { // HLG
	 // NOTE: HLG just maps ST.2084 output to HLG encoding. 
	 // TODO: Restructure if/else tree to minimize this redundancy.
		if (STRETCH_BLACK == true) {
			outputCV = Y_2_ST2084_f3(clamp(linCV_2_Y_f3(linearCV, Y_MAX, 0.0), 0.0, HALF_POS_INF));
		}
		else {
			outputCV = Y_2_ST2084_f3(linCV_2_Y_f3(linearCV, Y_MAX, Y_MIN));
		}
		outputCV = ST2084_2_HLG_1000nits_f3(outputCV);
	}

	if (LEGAL_RANGE == true) {
		outputCV = fullRange_to_smpteRange_f3(outputCV);
	}

	return outputCV;
}

float3 outputTransform
(
	float3 inValue,
	float Y_MIN,
	float Y_MID,
	float Y_MAX,
	FColorSpace DISPLAY_PRI,
	FColorSpace LIMITING_PRI,
	int EOTF,
	int SURROUND,
	bool STRETCH_BLACK = true,
	bool D60_SIM = false,
	bool LEGAL_RANGE = false
)
{
	/*
		NOTE: This is a bit of a hack - probably a more direct way to do this.
		TODO: Fix in future version
	*/
	TsParams PARAMS_DEFAULT = init_TsParams(Y_MIN, Y_MAX);
	float expShift = log2(inv_ssts(Y_MID, PARAMS_DEFAULT)) - log2(0.18);
	TsParams PARAMS = init_TsParams(Y_MIN, Y_MAX, expShift);

	return outputTransform(inValue, PARAMS, DISPLAY_PRI, LIMITING_PRI, EOTF, SURROUND, STRETCH_BLACK, D60_SIM, LEGAL_RANGE);
}

float3 invOutputTransform
(
	float3 inValue,
	float Y_MIN,
	float Y_MID,
	float Y_MAX,
	FColorSpace DISPLAY_PRI,
	FColorSpace LIMITING_PRI,
	int EOTF,
	int SURROUND,
	bool STRETCH_BLACK = true,
	bool D60_SIM = false,
	bool LEGAL_RANGE = false
)
{
	float3x3 DISPLAY_PRI_2_XYZ_MAT = DISPLAY_PRI.RGBtoXYZ;

	/*
		NOTE: This is a bit of a hack - probably a more direct way to do this.
		TODO: Update in accordance with forward algorithm.
	*/
	TsParams PARAMS_DEFAULT = init_TsParams(Y_MIN, Y_MAX);
	float expShift = log2(inv_ssts(Y_MID, PARAMS_DEFAULT)) - log2(0.18);
	TsParams PARAMS = init_TsParams(Y_MIN, Y_MAX, expShift);

	float3 outputCV = inValue;

	if (LEGAL_RANGE == true) {
		outputCV = smpteRange_to_fullRange_f3(outputCV);
	}

	// Inverse EOTF
	// 0: ST-2084 (PQ)
	// 1: BT.1886 (Rec.709/2020 settings)
	// 2: sRGB (mon_curve w/ presets)
	//    moncurve_r with gamma of 2.4 and offset of 0.055 matches the EOTF found in IEC 61966-2-1:1999 (sRGB)
	// 3: gamma 2.6
	// 4: linear (no EOTF)
	// 5: HLG
	float3 linearCV;
	if (EOTF == 0) {  // ST-2084 (PQ)
		if (STRETCH_BLACK == true) {
			linearCV = Y_2_linCV_f3(ST2084_2_Y_f3(outputCV), Y_MAX, 0.);
		}
		else {
			linearCV = Y_2_linCV_f3(ST2084_2_Y_f3(outputCV), Y_MAX, Y_MIN);
		}
	}
	else if (EOTF == 1) { // BT.1886 (Rec.709/2020 settings)
		linearCV = bt1886_f_f3(outputCV, 2.4, 1.0, 0.0);
	}
	else if (EOTF == 2) { // sRGB (mon_curve w/ presets)
		linearCV = moncurve_f_f3(outputCV, 2.4, 0.055);
	}
	else if (EOTF == 3) { // gamma 2.6
		linearCV = pow(outputCV, 2.6);
	}
	else if (EOTF == 4) { // linear
		linearCV = Y_2_linCV_f3(outputCV, Y_MAX, Y_MIN);
	}
	else if (EOTF == 5) { // HLG
		outputCV = HLG_2_ST2084_1000nits_f3(outputCV);
		if (STRETCH_BLACK == true) {
			linearCV = Y_2_linCV_f3(ST2084_2_Y_f3(outputCV), Y_MAX, 0.);
		}
		else {
			linearCV = Y_2_linCV_f3(ST2084_2_Y_f3(outputCV), Y_MAX, Y_MIN);
		}
	}

	// Un-scale
	if (D60_SIM == true) {
		/* TODO: The scale requires calling itself. Need an algorithm for this.
			Scale is same no matter the luminance.
			Currently using precalculated values for D65, DCI.
			If DCI, roll_white_fwd is used also.
		*/
		float SCALE = 1.0;
		if (DISPLAY_PRI.White == WhiteStandard_D65) { // D65
			SCALE = 0.96362;
			linearCV /= SCALE;
		}
		else if (DISPLAY_PRI.White == WhiteStandard_DCI) { // DCI
			SCALE = 0.96;
			linearCV[0] = roll_white_rev(linearCV[0] / SCALE, 0.918, 0.5);
			linearCV[1] = roll_white_rev(linearCV[1] / SCALE, 0.918, 0.5);
			linearCV[2] = roll_white_rev(linearCV[2] / SCALE, 0.918, 0.5);
		}

	}

	// Encoding primaries to CIE XYZ
	float3 XYZ = mul(DISPLAY_PRI_2_XYZ_MAT, linearCV);

	// Undo CAT from assumed observer adapted white point to ACES white point
	if (D60_SIM == false) {
		if (DISPLAY_PRI.White != WhiteStandard_D60) {
			XYZ = mul(D65_2_D60_CAT, XYZ);
		}
	}

	// Apply gamma adjustment to compensate for dim surround
	/*
		NOTE: This is more or less a placeholder block and is largely inactive
		in its current form. This section currently only applies for SDR, and
		even then, only in very specific cases.
		In the future it is fully intended for this module to be updated to
		support surround compensation regardless of luminance dynamic range. */
		/*
			TOD0: Come up with new surround compensation algorithm, applicable
			across all dynamic ranges and supporting dark/dim/normal surround.
		*/
	if (SURROUND == 0) { // Dark surround
		/*
		Current tone scale is designed for dark surround environment so no
		adjustment is necessary.
		*/
	}
	else if (SURROUND == 1) { // Dim surround
	 // INACTIVE for HDR and crudely implemented for SDR (see comment below)        
		if ((EOTF == 1) || (EOTF == 2) || (EOTF == 3)) {
			/*
			This uses a crude logical assumption that if the EOTF is BT.1886,
			sRGB, or gamma 2.6 that the data is SDR and so the SDR gamma
			compensation factor from v1.0 will apply.
			*/
			XYZ = dim_to_dark(XYZ); /*
			This uses a local dim_to_dark function that is designed to take in
			XYZ and return XYZ rather than AP1 as is currently in the functions
			in 'ACESlib.ODT_Common.ctl' */
		}
	}
	else if (SURROUND == 2) { // Normal surround
	 // INACTIVE - this does NOTHING
	}

	// XYZ to rendering primaries
	linearCV = mul(XYZ_2_AP1_MAT, XYZ);

	float3 rgbPost = linCV_2_Y_f3(linearCV, Y_MAX, Y_MIN);

	// Apply the inverse tonescale independently in rendering-space RGB
	float3 rgbPre = inv_ssts_f3(rgbPost, PARAMS);

	// RRT sweeteners
	float3 aces = inv_rrt_sweeteners(rgbPre);

	return aces;
}

/* --- Gamut Compress Parameters --- */
// Distance from achromatic which will be compressed to the gamut boundary
// Values calculated to encompass the encoding gamuts of common digital cinema cameras
static const float LIM_CYAN = 1.147;
static const float LIM_MAGENTA = 1.264;
static const float LIM_YELLOW = 1.312;

// Percentage of the core gamut to protect
// Values calculated to protect all the colors of the ColorChecker Classic 24 as given by
// ISO 17321-1 and Ohta (1997)
static const float THR_CYAN = 0.815;
static const float THR_MAGENTA = 0.803;
static const float THR_YELLOW = 0.880;

// Aggressiveness of the compression curve
static const float PWR = 1.2;



// Calculate compressed distance
float compress(float dist, float lim, float thr, float pwr, bool invert)
{
	float comprDist;
	float scl;
	float nd;
	float p;

	if (dist < thr) {
		comprDist = dist; // No compression below threshold
	}
	else {
		// Calculate scale factor for y = 1 intersect
		scl = (lim - thr) / pow(pow((1.0 - thr) / (lim - thr), -pwr) - 1.0, 1.0 / pwr);

		// Normalize distance outside threshold by scale factor
		nd = (dist - thr) / scl;
		p = pow(nd, pwr);

		if (!invert) {
			comprDist = thr + scl * nd / (pow(1.0 + p, 1.0 / pwr)); // Compress
		}
		else {
			if (dist > (thr + scl)) {
				comprDist = dist; // Avoid singularity
			}
			else {
				comprDist = thr + scl * pow(-(p / (p - 1.0)), 1.0 / pwr); // Uncompress
			}
		}
	}

	return comprDist;
}

float max_f3(float3 a)
{
	return max(a[0], max(a[1], a[2]));
}

float3 compressColor
(
	const float3 ACES,
	const bool invert = false
)
{
	// Convert to ACEScg
	float3 linAP1 = mul(AP0_2_AP1_MAT, ACES);

	// Achromatic axis
	float ach = max_f3(linAP1);

	// Distance from the achromatic axis for each color component aka inverse RGB ratios
	float3 comprLinAP1;
	///+ added any(ACES < 0): don't try to compress colors outside the AP0 color space: clamping will happen anyway during RRT/ODT, 
	// and this allows to keep proper results when testing with a synthetic chart
	if (ach < 1e-10f || any(ACES < 0)) {
		///-
		comprLinAP1 = linAP1;
	}
	else {
		float3 dist;
		dist[0] = (ach - linAP1[0]) / abs(ach);
		dist[1] = (ach - linAP1[1]) / abs(ach);
		dist[2] = (ach - linAP1[2]) / abs(ach);

		float3 comprDist = {
		compress(dist[0], LIM_CYAN, THR_CYAN, PWR, invert),
		compress(dist[1], LIM_MAGENTA, THR_MAGENTA, PWR, invert),
		compress(dist[2], LIM_YELLOW, THR_YELLOW, PWR, invert)
		};

		// Recalculate RGB from compressed distance and achromatic
		comprLinAP1 = float3(
			ach - comprDist[0] * abs(ach),
			ach - comprDist[1] * abs(ach),
			ach - comprDist[2] * abs(ach)
		);
	}

	// Convert back to ACES2065-1
	return mul(AP1_2_AP0_MAT, comprLinAP1);
}

