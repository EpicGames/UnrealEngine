// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ASTCompressionCommon.ush:
	Helpers for compute shader ASTC texture compression
=============================================================================*/

#pragma once

#include "BlockCompressionCommon.ush"
#include "BCCompressionCommon.ush"
#include "GammaCorrectionCommon.ush"
#include "IntegerSequenceEncoding.ush"


float MaxNorm3(float3 v)
{
	v = abs(v);
	return max(max(v.x, v.y), v.z);
}

float MaxNorm4(float4 v)
{
	v = abs(v);
	return max(max(max(v.x, v.y), v.z), v.w);
}

void PCAOptimizeEndpoints3f(in float4 BlockRGBA[16], inout float4 BlockMin, inout float4 BlockMax)
{
	// Compute mean of all texels (3D).
	float3 Mean = 0.0f;
	float m00 = 0.0f, m01 = 0.0f, m02 = 0.0f;
	float m11 = 0.0f, m12 = 0.0f, m22 = 0.0f;
	for (uint i = 0; i < 16; i++)
	{
		float3 c = BlockRGBA[i].rgb;
		Mean += c;
		m00 += c.x * c.x;
		m01 += c.x * c.y;
		m02 += c.x * c.z;
		m11 += c.y * c.y;
		m12 += c.y * c.z;
		m22 += c.z * c.z;
	}
	Mean /= 16.0f;

	m00 -= 16.0f * Mean.x * Mean.x;
	m01 -= 16.0f * Mean.x * Mean.y;
	m02 -= 16.0f * Mean.x * Mean.z;
	m11 -= 16.0f * Mean.y * Mean.y;
	m12 -= 16.0f * Mean.y * Mean.z;
	m22 -= 16.0f * Mean.z * Mean.z;

	// Build candidate vectors from the "rows" of the covariance matrix.
	float3 CovCol0 = float3(m00, m01, m02);
	float3 CovCol1 = float3(m01, m11, m12);
	float3 CovCol2 = float3(m02, m12, m22);

	// Choose the candidate with the largest norm.
	float3 CurrentAxis = CovCol0;
	float CurrentNorm = MaxNorm3(CovCol0);

	float CandidateNorm = MaxNorm3(CovCol1);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol1;
		CurrentNorm = CandidateNorm;
	}
	CandidateNorm = MaxNorm3(CovCol2);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol2;
		CurrentNorm = CandidateNorm;
	}
	
	if (CurrentNorm != 0.0f)
	{
		CurrentAxis = normalize(CurrentAxis);
		
		// Power-Method-style iteration to refine the principal axis.
		for (uint iteration = 0; iteration < 3; iteration++)
		{
			CurrentAxis = normalize(CovCol0 * CurrentAxis.x + CovCol1 * CurrentAxis.y + CovCol2 * CurrentAxis.z);
		}
	}
	else
	{
		CurrentAxis = float3(1.0f, 0.0f, 0.0f);
	}

	// Project each texel onto the principal axis to find minimum / maximum projections.
	float MinProj = 999999.0f;
	float MaxProj = -999999.0f;
	for (uint i = 0; i < 16; i++)
	{
		float Projection = dot(BlockRGBA[i].rgb - Mean, CurrentAxis);
		MinProj = min(MinProj, Projection);
		MaxProj = max(MaxProj, Projection);
	}

	// Reconstruct endpoints from the min / max projections, clamped to [0..1].
	BlockMin.rgb = saturate(Mean + CurrentAxis * MinProj);
	BlockMax.rgb = saturate(Mean + CurrentAxis * MaxProj);
}

void PCAOptimizeEndpoints3f_6x6(in float4 BlockRGBA[36], inout float3 BlockMin, inout float3 BlockMax)
{
	// Compute the mean and covariance of all texels (only RGB channels).
	float3 Mean = 0.0f;
	float m00 = 0.0f, m01 = 0.0f, m02 = 0.0f;
	float m11 = 0.0f, m12 = 0.0f, m22 = 0.0f;
	for (uint i = 0; i < 36; i++)
	{
		float3 c = BlockRGBA[i].rgb;
		Mean += c;
		m00 += c.x * c.x;
		m01 += c.x * c.y;
		m02 += c.x * c.z;
		m11 += c.y * c.y;
		m12 += c.y * c.z;
		m22 += c.z * c.z;
	}
	Mean /= 36.0f;
	
	// Compute the covariance components.
	m00 -= 36.0f * Mean.x * Mean.x;
	m01 -= 36.0f * Mean.x * Mean.y;
	m02 -= 36.0f * Mean.x * Mean.z;
	m11 -= 36.0f * Mean.y * Mean.y;
	m12 -= 36.0f * Mean.y * Mean.z;
	m22 -= 36.0f * Mean.z * Mean.z;
	
	// Build the candidate axis vectors from the covariance matrix.
	float3 CovCol0 = float3(m00, m01, m02);
	float3 CovCol1 = float3(m01, m11, m12);
	float3 CovCol2 = float3(m02, m12, m22);
	
	// Choose the candidate with the largest norm.
	float3 CurrentAxis = CovCol0;
	float CurrentNorm = MaxNorm3(CovCol0);
	float CandidateNorm = MaxNorm3(CovCol1);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol1;
		CurrentNorm = CandidateNorm;
	}
	CandidateNorm = MaxNorm3(CovCol2);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol2;
		CurrentNorm = CandidateNorm;
	}
	
	// Refine the principal axis using power iterations.
	if (CurrentNorm != 0.0f)
	{
		CurrentAxis = normalize(CurrentAxis);
		for (uint iteration = 0; iteration < 3; iteration++)
		{
			CurrentAxis = normalize(CovCol0 * CurrentAxis.x + CovCol1 * CurrentAxis.y + CovCol2 * CurrentAxis.z);
		}
	}
	else
	{
		CurrentAxis = float3(1.0f, 0.0f, 0.0f);
	}
	
	// Project each texel onto the principal axis to find the minimum and maximum projections.
	float MinProj = 999999.0f;
	float MaxProj = -999999.0f;
	for (uint i = 0; i < 36; i++)
	{
		float Projection = dot(BlockRGBA[i].rgb - Mean, CurrentAxis);
		MinProj = min(MinProj, Projection);
		MaxProj = max(MaxProj, Projection);
	}
	
	// Reconstruct the endpoints from the min and max projections, clamping results to [0,1].
	BlockMin = saturate(Mean + CurrentAxis * MinProj);
	BlockMax = saturate(Mean + CurrentAxis * MaxProj);
}

void PCAOptimizeEndpoints4f(in float4 BlockRGBA[16], inout float4 BlockMin, inout float4 BlockMax)
{
	// Compute mean over all texels
	float4 Mean = float4(0.0, 0.0, 0.0, 0.0);
	float m00 = 0.0, m01 = 0.0, m02 = 0.0, m03 = 0.0;
	float m11 = 0.0, m12 = 0.0, m13 = 0.0;
	float m22 = 0.0, m23 = 0.0;
	float m33 = 0.0;
	for (uint i = 0; i < 16; i++)
	{
		float4 c = BlockRGBA[i];
		Mean += c;
		m00 += c.x * c.x;
		m01 += c.x * c.y;
		m02 += c.x * c.z;
		m03 += c.x * c.w;
		m11 += c.y * c.y;
		m12 += c.y * c.z;
		m13 += c.y * c.w;
		m22 += c.z * c.z;
		m23 += c.z * c.w;
		m33 += c.w * c.w;
	}
	Mean /= 16.0;

	m00 -= 16.0 * Mean.x * Mean.x;
	m01 -= 16.0 * Mean.x * Mean.y;
	m02 -= 16.0 * Mean.x * Mean.z;
	m03 -= 16.0 * Mean.x * Mean.w;
	m11 -= 16.0 * Mean.y * Mean.y;
	m12 -= 16.0 * Mean.y * Mean.z;
	m13 -= 16.0 * Mean.y * Mean.w;
	m22 -= 16.0 * Mean.z * Mean.z;
	m23 -= 16.0 * Mean.z * Mean.w;
	m33 -= 16.0 * Mean.w * Mean.w;

	// Build candidate vectors from the symmetric 4x4 covariance matrix.
	float4 CovCol0 = float4(m00, m01, m02, m03);
	float4 CovCol1 = float4(m01, m11, m12, m13);
	float4 CovCol2 = float4(m02, m12, m22, m23);
	float4 CovCol3 = float4(m03, m13, m23, m33);

	// Evaluate max norms of the candidates.
	float4 CurrentAxis = CovCol0;
	float CurrentNorm = MaxNorm4(CovCol0);

	float CandidateNorm = MaxNorm4(CovCol1);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol1;
		CurrentNorm = CandidateNorm;
	}
	CandidateNorm = MaxNorm4(CovCol2);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol2;
		CurrentNorm = CandidateNorm;
	}
	CandidateNorm = MaxNorm4(CovCol3);
	if (CandidateNorm > CurrentNorm)
	{
		CurrentAxis = CovCol3;
		CurrentNorm = CandidateNorm;
	}
	if (CurrentNorm > 0.0)
	{
		CurrentAxis = normalize(CurrentAxis);
		// Refine with a few iterations of the power method.
		for (uint iteration = 0; iteration < 5; iteration++)
		{
			CurrentAxis = normalize(CovCol0 * CurrentAxis.x + CovCol1 * CurrentAxis.y + CovCol2 * CurrentAxis.z + CovCol3 * CurrentAxis.w);
		}
	}
	else
	{
		CurrentAxis = float4(1.0, 0.0, 0.0, 0.0);
	}
	
	// Project each texel (in 4D) onto Axis to locate end‐points.
	float minProj = 1e10;
	float maxProj = -1e10;
	for (uint i = 0; i < 16; i++)
	{
		float proj = dot(BlockRGBA[i] - Mean, CurrentAxis);
		minProj = min(minProj, proj);
		maxProj = max(maxProj, proj);
	}

	BlockMin = saturate(Mean + CurrentAxis * minProj);
	BlockMax = saturate(Mean + CurrentAxis * maxProj);
}

void LLSOptimizeEndpoints4f16(float4 Texels[16], uint WeightRange, inout float4 BlockMin, inout float4 BlockMax)
{
	float4 BlockVector = BlockMax - BlockMin;
	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	float4 TexelSum = 0.0f;
	float4 BetaTexelSum = 0.0f;
	float BetaSum = 0.0f;
	float BetaSqSum = 0.0f;

	for (uint i = 0; i < 16; ++i)
	{
		float Pos = dot(Texels[i], BlockVector);
		float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
		uint Index = (uint) (NormalizedPos * WeightRange + 0.5f);
		float Beta = Index / (float) WeightRange;

		TexelSum += Texels[i];
		BetaTexelSum += Texels[i] * Beta;
		BetaSum += Beta;
		BetaSqSum += Beta * Beta;
	}

	float4 AlphaTexelSum = TexelSum - BetaTexelSum;
	float AlphaSqSum = 16.0f - 2 * BetaSum + BetaSqSum;
	float AlphaBetaSum = BetaSum - BetaSqSum;

	float Det = AlphaSqSum * BetaSqSum - AlphaBetaSum * AlphaBetaSum;
	if (abs(Det) > 0.1f)
	{
		float RcpDet = 1.0f / Det;
		BlockMin = saturate((AlphaTexelSum * BetaSqSum - BetaTexelSum * AlphaBetaSum) * RcpDet);
		BlockMax = saturate((BetaTexelSum * AlphaSqSum - AlphaTexelSum * AlphaBetaSum) * RcpDet);
	}
}

void LLSOptimizeEndpoints3f16(float4 Texels[16], uint WeightRange, inout float3 BlockMin, inout float3 BlockMax)
{
	float3 BlockVector = BlockMax - BlockMin;
	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	float3 TexelSum     = float3(0.0f, 0.0f, 0.0f);
	float3 BetaTexelSum = float3(0.0f, 0.0f, 0.0f);
	float BetaSum  = 0.0f;
	float BetaSqSum = 0.0f;

	for (uint i = 0; i < 16; ++i)
	{
		float Pos = dot(Texels[i].rgb, BlockVector);
		float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
		uint Index = (uint)(NormalizedPos * WeightRange + 0.5f);
		float Beta = Index / (float)WeightRange;

		TexelSum     += Texels[i].rgb;
		BetaTexelSum += Texels[i].rgb * Beta;
		BetaSum    += Beta;
		BetaSqSum  += Beta * Beta;
	}

	float3 AlphaTexelSum = TexelSum - BetaTexelSum;
	float AlphaSqSum = 16.0f - 2.0f * BetaSum + BetaSqSum;
	float AlphaBetaSum = BetaSum - BetaSqSum;

	float Det = AlphaSqSum * BetaSqSum - AlphaBetaSum * AlphaBetaSum;
	if (abs(Det) > 0.1f)
	{
		float RcpDet = 1.0f / Det;
		BlockMin = saturate((AlphaTexelSum * BetaSqSum - BetaTexelSum * AlphaBetaSum) * RcpDet);
		BlockMax = saturate((BetaTexelSum * AlphaSqSum - AlphaTexelSum * AlphaBetaSum) * RcpDet);
	}
}

// https://registry.khronos.org/DataFormat/specs/1.3/dataformat.1.3.html See 23.10 Block Mode for more details
uint BuildBlockMode(uint WeightQuantMethod, uint BlockSize)
{
	uint H = (BlockSize - 2) & 0x3;
	uint W = (BlockSize - 4) & 0x3;

	uint DualPlane = 0;
	uint Precision = (WeightQuantMethod < 6) ? 0 : 1;
	uint Range = (WeightQuantMethod % 6) + 2;

	uint BlockMode = (Range >> 1) & 0x3;
	BlockMode |= (Range & 0x1) << 4;
	BlockMode |= H << 5;
	BlockMode |= W << 7;
	BlockMode |= Precision << 9;
	BlockMode |= DualPlane << 10;
	return BlockMode;
}

void EncodeWeights3f(in float4 BlockRGBA[16], uint WeightRange, uint WeightQuantMethod, float4 BlockMin, float4 BlockMax, inout uint4 BlockBuffer)
{
	float3 BlockVector = BlockMax.rgb - BlockMin.rgb;

	float EndPoint0Pos = dot(BlockMin.rgb, BlockVector);
	float EndPoint1Pos = dot(BlockMax.rgb, BlockVector);

	uint WeightedBlocks[16];
	// Quantize the weights using the endpoings and the Weight Range
	for (uint ColorIndex = 0; ColorIndex < 16; ++ColorIndex)
	{
		float Pos = dot(BlockRGBA[ColorIndex].rgb, BlockVector);
		float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
		uint QuantizedWeight = (uint) (NormalizedPos * WeightRange + 0.5f);

		uint WeightIndex = WeightQuantMethod * WEIGHT_QUANTIZE_NUM + QuantizedWeight;
		WeightedBlocks[ColorIndex] = AstcParameters.ScrambleTable[WeightIndex];
	}

	uint4 QuantWeights = uint4(0, 0, 0, 0);
	uint QuantOffset = 0;
	EncodeWeightsQuant16(WeightQuantMethod, WeightedBlocks, QuantWeights, QuantOffset);
	
	BlockBuffer.w = reversebits(QuantWeights.x);
	BlockBuffer.z = reversebits(QuantWeights.y);
	BlockBuffer.y = reversebits(QuantWeights.z);
}

void EncodeWeights4f(in float4 BlockRGBA[16], uint WeightRange, uint WeightQuantMethod, float4 BlockMin, float4 BlockMax, inout uint4 BlockBuffer)
{
	float4 BlockVector = BlockMax - BlockMin;

	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	uint WeightedBlocks[16];
	// Quantize the weights using the endpoings and the Weight Range
	for (uint ColorIndex = 0; ColorIndex < 16; ++ColorIndex)
	{
		float Pos = dot(BlockRGBA[ColorIndex], BlockVector);
		float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
		uint QuantizedWeight = (uint) (NormalizedPos * WeightRange + 0.5f);

		uint WeightIndex = WeightQuantMethod * WEIGHT_QUANTIZE_NUM + QuantizedWeight;
		WeightedBlocks[ColorIndex] = AstcParameters.ScrambleTable[WeightIndex];
	}

	uint4 QuantWeights = uint4(0, 0, 0, 0);
	uint QuantOffset = 0;
	EncodeWeightsQuant16(WeightQuantMethod, WeightedBlocks, QuantWeights, QuantOffset);
	
	BlockBuffer.w = reversebits(QuantWeights.x);
	BlockBuffer.z = reversebits(QuantWeights.y);
	BlockBuffer.y = reversebits(QuantWeights.z);
}

uint4 Compress_ASTC_RGBA_4x4(in float4 BlockRGBA[16])
{
	// Compute initial endpoints
	float4 BlockMin = BlockRGBA[0];
	float4 BlockMax = BlockRGBA[0];
	for (uint TexelIndex = 1; TexelIndex < 16; ++TexelIndex)
	{
		BlockMin = min(BlockMin, BlockRGBA[TexelIndex]);
		BlockMax = max(BlockMax, BlockRGBA[TexelIndex]);
	}

	uint BlockSize = 4;
	
	// Note: This is just a way to encode a void-extent like block - doesn't actually encode void-extent.
	if (all(BlockMin == BlockMax))
	{
		uint ColorEndpointMode = 12; // LDR RGBA, direct
		uint WeightQuantMethod = QUANT_4;
		uint EndpointQuantMethod = QUANT_256;
		uint WeightRange = 4u - 1u;
		uint EndpointRange = 256u - 1u;

		uint PartitionCount = 1;

		uint BlockMode = BuildBlockMode(WeightQuantMethod, BlockSize);
		BlockMode |= (PartitionCount - 1) << 11;
		BlockMode |= ColorEndpointMode << 13;
		
		uint4 BlockBuffer = uint4(BlockMode, 0, 0, 0);
		uint BlockOffset = 11 + 2 + 4;

		uint4 QuantizeEndpoint = uint4(BlockMin * EndpointRange + 0.5f);

		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.x, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.y, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.z, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.w, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		return BlockBuffer;
	}

	// Calculate the endpoint bit range
	bool NoAlpha = BlockMin.a == 1.0f;

	uint EndpointQuantMethod = QUANT_256;
	uint EndpointRange = 256u - 1u;

	uint4 BlockBuffer = uint4(0, 0, 0, 0);
	uint BlockOffset = 11 + 2 + 4; // BlockMode + (PartitionCount-1) + CEM

	if (NoAlpha)
	{
		uint ColorEndpointMode = 8; // LDR RGB, direct
		uint WeightQuantMethod = QUANT_12;
		uint WeightRange = 12u - 1u;
		uint PartitionCount = 1;

		uint BlockMode = BuildBlockMode(WeightQuantMethod, BlockSize);
		BlockMode |= (PartitionCount - 1) << 11;
		BlockMode |= ColorEndpointMode << 13;
		BlockBuffer.x = BlockMode;
	
#ifdef ASTC_HIGH_PROFILE
		PCAOptimizeEndpoints3f(BlockRGBA, BlockMin, BlockMax);
#endif
		//LLSOptimizeEndpoints3f16(BlockRGBA, WeightRange, BlockMin, BlockMax);
	
		// Inset the min/max
		// See https://developer.download.nvidia.com/whitepapers/2007/Real-Time-YCoCg-DXT-Compression/Real-Time%20YCoCg-DXT%20Compression.pdf
		{
			float3 offset = (1.0f / ((WeightRange + 1) * 4.f)) * (BlockMax.rgb - BlockMin.rgb);
			BlockMax.rgb = ceil((BlockMax.rgb - offset) * 255.0f) / 255.0f;
			BlockMin.rgb = floor((BlockMin.rgb + offset) * 255.0f) / 255.0f;
		}

		float SumMin = BlockMin.x + BlockMin.y + BlockMin.z;
		float SumMax = BlockMax.x + BlockMax.y + BlockMax.z;
		if (SumMin > SumMax)
		{
			float4 tmp = BlockMin;
			BlockMin = BlockMax;
			BlockMax = tmp;
		}

		EncodeWeights3f(BlockRGBA, WeightRange, WeightQuantMethod, BlockMin, BlockMax, BlockBuffer);

		uint4 QuantizeEndpoint0 = uint4(BlockMin * EndpointRange + 0.5f);
		uint4 QuantizeEndpoint1 = uint4(BlockMax * EndpointRange + 0.5f);

		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.x, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.x, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.y, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.y, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.z, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.z, 8);
	}
	else
	{
		uint ColorEndpointMode = 12; // LDR RGBA, direct
		uint WeightQuantMethod = QUANT_6;
		uint WeightRange = 6u - 1u;
		uint PartitionCount = 1;

		uint BlockMode = BuildBlockMode(WeightQuantMethod, BlockSize);
		BlockMode |= (PartitionCount - 1) << 11;
		BlockMode |= ColorEndpointMode << 13;
		BlockBuffer.x = BlockMode;

#ifdef ASTC_HIGH_PROFILE
		PCAOptimizeEndpoints4f(BlockRGBA, BlockMin, BlockMax);
#endif
		//LLSOptimizeEndpoints4f16(BlockRGBA, WeightRange, BlockMin, BlockMax);
	
		// Inset the min/max
		// See https://developer.download.nvidia.com/whitepapers/2007/Real-Time-YCoCg-DXT-Compression/Real-Time%20YCoCg-DXT%20Compression.pdf
		{
			float3 offset = (1.0f / ((WeightRange + 1) * 4.f)) * (BlockMax.rgb - BlockMin.rgb);
			BlockMax.rgb = ceil((BlockMax.rgb - offset) * 255.0f) / 255.0f;
			BlockMin.rgb = floor((BlockMin.rgb + offset) * 255.0f) / 255.0f;
		}

		float SumMin = BlockMin.x + BlockMin.y + BlockMin.z;
		float SumMax = BlockMax.x + BlockMax.y + BlockMax.z;
		if (SumMin > SumMax)
		{
			float4 tmp = BlockMin;
			BlockMin = BlockMax;
			BlockMax = tmp;
		}

		EncodeWeights4f(BlockRGBA, WeightRange, WeightQuantMethod, BlockMin, BlockMax, BlockBuffer);
		uint4 QuantizeEndpoint0 = uint4(BlockMin * EndpointRange + 0.5f);
		uint4 QuantizeEndpoint1 = uint4(BlockMax * EndpointRange + 0.5f);

		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.x, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.x, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.y, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.y, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.z, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.z, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint0.w, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint1.w, 8);
	}
		
	return BlockBuffer;
}

uint BuildBlockMode2(uint WeightQuantMethod, uint BlockX, uint BlockY)
{
	uint H = (BlockX - 6) & 0x3;
	uint W = (BlockY - 6) & 0x3;

	uint Range = (WeightQuantMethod % 6) + 2;
	uint BlockMode = 0x100;
	BlockMode |= (Range & 0x1) << 4;
	BlockMode |= ((Range >> 1) & 0x3) << 2;
	BlockMode |= H << 9;
	BlockMode |= W << 5;
	return BlockMode;
}

void EncodeWeights3f_6x6(in float4 BlockRGBA[36], uint WeightRange, uint WeightQuantMethod, float3 BlockMin, float3 BlockMax, inout uint4 BlockBuffer)
{
	float3 BlockVector = BlockMax - BlockMin;

	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	uint WeightedBlocks[36];
	// Quantize the weights using the endpoints and the Weight Range
	for (uint ColorIndex = 0; ColorIndex < 36; ++ColorIndex)
	{
		float Pos = dot(BlockRGBA[ColorIndex].rgb, BlockVector);
		float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
		uint QuantizedWeight = (uint) (NormalizedPos * WeightRange + 0.5f);

		uint WeightIndex = WeightQuantMethod * WEIGHT_QUANTIZE_NUM + QuantizedWeight;
		WeightedBlocks[ColorIndex] = AstcParameters.ScrambleTable[WeightIndex];
	}

	uint4 QuantWeights = uint4(0, 0, 0, 0);
	uint QuantOffset = 0;
	EncodeWeightsQuant36(WeightQuantMethod, WeightedBlocks, QuantWeights, QuantOffset);
	
	BlockBuffer.w |= reversebits(QuantWeights.x);
	BlockBuffer.z |= reversebits(QuantWeights.y);
	BlockBuffer.y |= reversebits(QuantWeights.z);
}

uint4 Compress_ASTC_RGB_6x6(in float4 BlockRGBA[36])
{
	// Compute initial endpoints
	float3 BlockMin = BlockRGBA[0].rgb;
	float3 BlockMax = BlockRGBA[0].rgb;
	for (uint TexelIndex = 1; TexelIndex < 36; ++TexelIndex)
	{
		BlockMin = min(BlockMin, BlockRGBA[TexelIndex].rgb);
		BlockMax = max(BlockMax, BlockRGBA[TexelIndex].rgb);
	}

	// Note: This is just a way to encode a void-extent like block - doesn't actually encode void-extent.
	if (all(BlockMin == BlockMax))
	{
		uint ColorEndpointMode = 8; // LDR RGBA, direct
		uint WeightQuantMethod = QUANT_4;
		uint EndpointQuantMethod = QUANT_256;
		uint WeightRange = 4u - 1u;
		uint EndpointRange = 256u - 1u;

		uint PartitionCount = 1;

		uint BlockMode = BuildBlockMode2(WeightQuantMethod, 4, 4);
		BlockMode |= (PartitionCount - 1) << 11;
		BlockMode |= ColorEndpointMode << 13;
		
		uint4 BlockBuffer = uint4(BlockMode, 0, 0, 0);
		uint BlockOffset = 11 + 2 + 4;

		uint3 QuantizeEndpoint = uint3(BlockMin * EndpointRange + 0.5f);

		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.x, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.y, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		WriteBits(BlockBuffer, BlockOffset, QuantizeEndpoint.z, 8);
		WriteBits(BlockBuffer, BlockOffset, 0xFF, 8);
		return BlockBuffer;
	}
	

	uint4 BlockBuffer = uint4(0, 0, 0, 0);
	uint BlockOffset = 11 + 2 + 4;

	uint EndpointQuantMethod = QUANT_80;
	uint EndpointRange = 80u - 1u;

	uint ColorEndpointMode = 8; // LDR RGB, direct
	uint WeightQuantMethod = QUANT_4;
	uint WeightRange = 4u - 1u;
	uint PartitionCount = 1;

	uint BlockMode = BuildBlockMode2(WeightQuantMethod, 6, 6);
	BlockMode |= (PartitionCount - 1) << 11;
	BlockMode |= ColorEndpointMode << 13;
	BlockBuffer.x = BlockMode;


#ifdef ASTC_HIGH_PROFILE
	PCAOptimizeEndpoints3f_6x6(BlockRGBA, BlockMin, BlockMax);
#endif

	// Inset the min/max
	// See https://developer.download.nvidia.com/whitepapers/2007/Real-Time-YCoCg-DXT-Compression/Real-Time%20YCoCg-DXT%20Compression.pdf
	{
		float3 offset = (1.0f / ((WeightRange + 1) * 4.f)) * (BlockMax - BlockMin);
		BlockMax = ceil((BlockMax - offset) * (float) EndpointRange) / (float) EndpointRange;
		BlockMin = floor((BlockMin + offset) * (float) EndpointRange) / (float) EndpointRange;
	}

	float SumMin = BlockMin.x + BlockMin.y + BlockMin.z;
	float SumMax = BlockMax.x + BlockMax.y + BlockMax.z;
	if (SumMin > SumMax)
	{
		float3 tmp = BlockMin;
		BlockMin = BlockMax;
		BlockMax = tmp;
	}

	EncodeWeights3f_6x6(BlockRGBA, WeightRange, WeightQuantMethod, BlockMin, BlockMax, BlockBuffer);

	uint3 QuantizeEndpoint0 = uint3(BlockMin * 255.0f + 0.5f);
	uint3 QuantizeEndpoint1 = uint3(BlockMax * 255.0f + 0.5f);

	uint EndpointNumbers[6] =
	{
		AstcParameters.ColorScrambleTable80[QuantizeEndpoint0.x], AstcParameters.ColorScrambleTable80[QuantizeEndpoint1.x],
		AstcParameters.ColorScrambleTable80[QuantizeEndpoint0.y], AstcParameters.ColorScrambleTable80[QuantizeEndpoint1.y],
		AstcParameters.ColorScrambleTable80[QuantizeEndpoint0.z], AstcParameters.ColorScrambleTable80[QuantizeEndpoint1.z]
	};

	EncodeEndpointsQuant6(EndpointQuantMethod, EndpointNumbers, BlockBuffer, BlockOffset);
	return BlockBuffer;
}

uint4 CompressBlock_ASTC_RGBA(in float3 BlockRGB[16], in float BlockA[16])
{
	float4 B[16];
	[unroll]
	for (int i = 0; i < 16; ++i)
	{
		B[i] = float4(BlockRGB[i], BlockA[i]);
	}
	return Compress_ASTC_RGBA_4x4(B);
}

uint4 CompressBlock_ASTC_SRGBA(in float3 BlockRGB[16], in float BlockA[16])
{
	float4 B[16];
	[unroll]
	for (int i = 0; i < 16; ++i)
	{
		B[i] = float4(LinearToSrgb(BlockRGB[i]), BlockA[i]);
	}
	return Compress_ASTC_RGBA_4x4(B);
}

uint4 CompressBlock_ASTC_YCoCg(in float3 BlockRGB[16])
{
	float4 B[16];
	[unroll]
	for (int i = 0; i < 16; ++i)
	{
		float3 YCoCg = RGB2YCoCg(BlockRGB[i]);
		B[i] = float4(YCoCg.yz, 0.0, YCoCg.x);
	}
	return Compress_ASTC_RGBA_4x4(B);
}

