// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostprocessAmbient.usf: To apply a ambient cubemap as a postprocess
=============================================================================*/

#define APPLY_SKY_SHADOWING 0
#define USE_DEFAULT_ENV_LIGHTING_INPUT 0
#define USE_SUBSTRATE_ENV_LIGHTING_COMMON 1
//#define ENABLE_DYNAMIC_SKY_LIGHT 1
#define ENABLE_DYNAMIC_SKY_LIGHT 0
#define ENABLE_SIMPLE_ENV_LIGHT 1

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "DeferredShadingCommon.ush"
#include "CubemapCommon.ush"
#include "Random.ush"
#include "BRDF.ush"
#include "MonteCarlo.ush"
#include "ShadingModels.ush"
#include "SceneTextureParameters.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

#define IMPORTANCE_SAMPLE		0

Texture2D AmbientOcclusionTexture;
SamplerState AmbientOcclusionSampler;

#if SUBSTRATE_ENABLED

// Override the default lighting input used for the env. lighting to sample the 'custom composite env. lighting'
float3 GetEnvDiffuseLighting(float3 InBentNormal)
{ 
	const float AbsoluteDiffuseMip = AmbientCubemapMipAdjust.z;
	return TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, InBentNormal, AbsoluteDiffuseMip).rgb * AmbientCubemapColor.rgb;
}

float3 GetEnvSpecularLighting(
	float CompositeAlpha,
	float3 TranslatedWorldPosition,
	float3 SpecularDirection,
	float SpecularSafeRoughness,
	float IndirectIrradiance,
	float IndirectSpecularOcclusion,
	float3 ExtraIndirectSpecular,
	uint NumCulledReflectionCaptures,
	uint CaptureDataStartIndex)
{
	const float SpecularMip = ComputeCubemapMipFromRoughness(SpecularSafeRoughness, AmbientCubemapMipAdjust.w);
	return TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, SpecularDirection, SpecularMip).rgb * AmbientCubemapColor.rgb;
}

#include "ReflectionEnvironmentShared.ush"
#include "SkyLightingDiffuseShared.ush"
#include "Substrate/Substrate.ush"
#include "Substrate/SubstrateEvaluation.ush"
#include "Substrate/SubstrateLightingCommon.ush"
#include "Substrate/SubstrateRead.ush"

void MainPS(
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	, out float3 OutOpaqueRoughRefractionSceneColor : SV_Target1
	, out float3 OutSubSurfaceSceneColor : SV_Target2
    #elif SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	, out float OutSeparatedSubsurfaceDiffuse : SV_Target1
	#endif
	)
{
	const int2 PixelPos = int2(SvPosition.xy);
	const float2 BufferUV = SvPositionToBufferUV(SvPosition);
	const float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;

	OutColor = 0;
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutOpaqueRoughRefractionSceneColor = 0.0f;
	OutSubSurfaceSceneColor = 0.0f;
#endif

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bMaterialIsLit = SubstratePixelHeader.ClosureCount > 0;	// This test is also enough to exclude sky pixels
	#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV, true);
	const FGBufferData GBuffer = ScreenSpaceData.GBuffer;
	const bool bMaterialIsLit = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
	#endif

	BRANCH
	if (bMaterialIsLit)
	{
		float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
		float SceneDepth = ConvertFromDeviceZ(DeviceZ);

		float3 TranslatedWorldPosition = mul(float4(ScreenPosition * SceneDepth, SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
		float3 CameraToPixel = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
		float3 V = -CameraToPixel;

		// Sample the ambient occlusion that is dynamically generated every frame.
		const float AmbientOcclusion = AmbientOcclusionTexture.SampleLevel(AmbientOcclusionSampler, BufferUV, 0).r;

		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);
		FSubstrateDeferredLighting SubstrateLighting = GetInitialisedSubstrateDeferredLighting();

		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
		Substrate_for (uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
		#else
		uint ClosureIndex = 0;
		#endif
		{
			#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
			FSubstrateBSDF BSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V);
			float CombinedScreenAndMaterialAO = SubstrateGetAO(SubstratePixelHeader) * AmbientOcclusion;
			FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion = SubstrateGetIrradianceAndAO(SubstratePixelHeader);
			#else
			FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(GBufferBSDF.BSDFTangentBasis, GBufferBSDF.BSDF, V, V/*L*/, false/*bHasValidL*/);
			float CombinedScreenAndMaterialAO = GBufferBSDF.BSDFAO;
			FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion = SubstrateGetIrradianceAndAO(ScreenSpaceData.GBuffer);
			#endif

			const float3 BSDFThroughput = LuminanceWeight(SubstrateBSDFContext, SubstrateBSDFContext.BSDF); // Use the reflected direction

			// Evaluate environment lighting
			FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);

			float3 DiffuseLighting = 0;
			float3 SpecularLighting = 0;
			SubstrateEnvLightingCommon(
				SubstrateEnvLight,
				SubstrateIrradianceAndOcclusion,
				SubstrateBSDFContext,
				SubstrateBSDFContext.BSDF,
				SubstrateEnvLight.DiffuseNormal,
				BSDFThroughput,
				0 /*CaptureDataStartIndex*/,
				0 /*NumCulledReflectionCaptures*/,
				AmbientOcclusion,
				1.0f /*CloudVolumetricAOShadow*/,
				1.0f /*TopLayerSpecularContributionFactor*/,
				TranslatedWorldPosition,
				CombinedScreenAndMaterialAO,
				DiffuseLighting,
				SpecularLighting);

			FLightAccumulator Out = (FLightAccumulator)0;
			LightAccumulator_AddSplit(Out, DiffuseLighting, SpecularLighting, DiffuseLighting, 1.f/*Legacy does not apply View.PreExposure*/, SubstrateEnvLight.bPostProcessSubsurface);
			AccumulateSubstrateDeferredLighting(SubstrateLighting, Out, SubstrateEnvLight.bPostProcessSubsurface, BSDF_GETISTOPLAYER(SubstrateBSDFContext.BSDF));
		}

		OutColor += SubstrateLighting.SceneColor;
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		OutOpaqueRoughRefractionSceneColor += SubstrateLighting.OpaqueRoughRefractionSceneColor;
		OutSubSurfaceSceneColor += SubstrateLighting.SubSurfaceSceneColor;
#endif

#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && !SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif
	}
}

#else // SUBSTRATE_ENABLED

float3 DiffuseIBL( uint2 Random, float3 DiffuseColor, float Roughness, float3 N, float3 V )
{
	N = normalize( N );
	V = normalize( V );

	float3 DiffuseLighting = 0;
	
	float NoV = saturate( dot( N, V ) );

	const uint NumSamples = 32;
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );
		float3 L = TangentToWorld( CosineSampleHemisphere( E ).xyz, N );
		float3 H = normalize(V + L);

		float NoL = saturate( dot( N, L ) );
		float NoH = saturate( dot( N, H ) );
		float VoH = saturate( dot( V, H ) );

		if( NoL > 0 )
		{
			float3 SampleColor = AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb;

			float FD90 = ( 0.5 + 2 * VoH * VoH ) * Roughness;
			//float FD90 = 0.5 + 2 * VoH * VoH * Roughness;
			float FdV = 1 + (FD90 - 1) * pow( 1 - NoV, 5 );
			float FdL = 1 + (FD90 - 1) * pow( 1 - NoL, 5 );

#if 1
			// lambert = DiffuseColor * NoL / PI
			// pdf = NoL / PI
			DiffuseLighting += SampleColor * DiffuseColor * FdV * FdL * ( 1 - 0.3333 * Roughness );
#else
			DiffuseLighting += SampleColor * DiffuseColor;
#endif
		}
	}

	return DiffuseLighting / NumSamples;
}

float3 SpecularIBL( uint2 Random, float3 SpecularColor, float Roughness, float3 N, float3 V )
{
	float3 SpecularLighting = 0;

	const uint NumSamples = 32;
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );
		float3 H = TangentToWorld( ImportanceSampleGGX( E, Pow4(Roughness) ).xyz, N );
		float3 L = 2 * dot( V, H ) * H - V;

		float NoV = saturate( dot( N, V ) );
		float NoL = saturate( dot( N, L ) );
		float NoH = saturate( dot( N, H ) );
		float VoH = saturate( dot( V, H ) );
		
		if( NoL > 0 )
		{
			float3 SampleColor = AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb;

			float Vis = Vis_SmithJointApprox( Pow4(Roughness), NoV, NoL );
			float Fc = pow( 1 - VoH, 5 );
			float3 F = (1 - Fc) * SpecularColor + Fc;

			// Incident light = SampleColor * NoL
			// Microfacet specular = D*G*F / (4*NoL*NoV) = D*Vis*F
			// pdf = D * NoH / (4 * VoH)
			SpecularLighting += SampleColor * F * ( NoL * Vis * (4 * VoH / NoH) );
		}
	}

	return SpecularLighting / NumSamples;
}


float3 ImageBasedLightingMIS( FGBufferData GBuffer, float3 V, float3 N, uint2 Random )
{
	float3 Lighting = 0;

	float Roughness1 = GBuffer.Roughness;
	float Roughness2 = 0.1;
	
	uint NumSamples[] =
	{
		16,
		16,
		0,
	};

	UNROLL
	for( uint Set = 0; Set < 3; Set++ )
	{
		LOOP
		for( uint i = 0; i < NumSamples[ Set ]; i++ )
		{
			float2 E = Hammersley( i, NumSamples[ Set ], Random );
			
			float3 L, H;
			if( Set == 0 )
			{
				L = TangentToWorld( CosineSampleHemisphere( E ).xyz, N );
				H = normalize(V + L);
			}
			else if( Set == 1 )
			{
				H = TangentToWorld( ImportanceSampleGGX( E, Pow4(Roughness1) ).xyz, N );
				L = 2 * dot( V, H ) * H - V;
			}
			else
			{
				H = TangentToWorld( ImportanceSampleGGX( E, Pow4(Roughness2) ).xyz, N );
				L = 2 * dot( V, H ) * H - V;
			}

			float NoL = saturate( dot(N, L) );
			float NoH = saturate( dot(N, H) );
			float VoH = saturate( dot(V, H) );
		
			if( NoL > 0 && VoH > 0 )
			{
				float3 SampleColor = AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb;

				float PDF[] =
				{
					NoL / PI,
					D_GGX( Pow4(Roughness1), NoH ) * NoH / (4 * VoH),
					D_GGX( Pow4(Roughness2), NoH ) * NoH / (4 * VoH),
				};

				// MIS balance heuristic
				float InvWeight = 0;
				UNROLL for( uint j = 0; j < 3; j++ )
				{
					InvWeight += PDF[j] * NumSamples[j];
				}
				float Weight = rcp( InvWeight );

				FShadowTerms Shadow = { 1, 1, 1, InitHairTransmittanceData() };
				FDirectLighting LightingSample = EvaluateBxDF( GBuffer, N, V, L, NoL, Shadow );

				Lighting += SampleColor * Weight * ( LightingSample.Diffuse + LightingSample.Specular + LightingSample.Transmission );
			}
		}
	}

	return Lighting;
}

float3 FilterEnvMap( uint2 Random, float Roughness, float3 N, float3 V )
{
	float3 FilteredColor = 0;
	float Weight = 0;

	const uint NumSamples = 64;
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );
		float3 H = TangentToWorld( ImportanceSampleGGX( E, Pow4(Roughness) ).xyz, N );
		float3 L = 2 * dot( V, H ) * H - V;

		float NoL = saturate( dot( N, L ) );
		if( NoL > 0 )
		{
			FilteredColor += AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb * NoL;
			Weight += NoL;
		}
	}

	return FilteredColor / max( Weight, 0.001 );
}

float3 PrefilterEnvMap( uint2 Random, float Roughness, float3 R )
{
	float3 FilteredColor = 0;
	float Weight = 0;
		
	const uint NumSamples = 64;
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );
		float3 H = TangentToWorld( ImportanceSampleGGX( E, Pow4(Roughness) ).xyz, R );
		float3 L = 2 * dot( R, H ) * H - R;

		float NoL = saturate( dot( R, L ) );
		if( NoL > 0 )
		{
			FilteredColor += AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb * NoL;
			Weight += NoL;
		}
	}

	return FilteredColor / max( Weight, 0.001 );
}

float3 IntegrateBRDF( uint2 Random, float Roughness, float NoV )
{
	float3 V;
	V.x = sqrt( 1.0f - NoV * NoV );	// sin
	V.y = 0;
	V.z = NoV;						// cos

	float A = 0;
	float B = 0;
	float C = 0;

	const uint NumSamples = 64;
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );

		{
			float3 H = ImportanceSampleGGX( E, Pow4(Roughness) ).xyz;
			float3 L = 2 * dot( V, H ) * H - V;

			float NoL = saturate( L.z );
			float NoH = saturate( H.z );
			float VoH = saturate( dot( V, H ) );

			if( NoL > 0 )
			{
				float a = Square( Roughness );
				float a2 = a*a;
				float Vis = Vis_SmithJointApprox( a2, NoV, NoL );
				float Vis_SmithV = NoL * sqrt( NoV * (NoV - NoV * a2) + a2 );
				float Vis_SmithL = NoV * sqrt( NoL * (NoL - NoL * a2) + a2 );
				//float Vis = 0.5 * rcp( Vis_SmithV + Vis_SmithL );

				// Incident light = NoL
				// pdf = D * NoH / (4 * VoH)
				// NoL * Vis / pdf
				float NoL_Vis_PDF = NoL * Vis * (4 * VoH / NoH);

				float Fc = pow( 1 - VoH, 5 );
				A += (1 - Fc) * NoL_Vis_PDF;
				B += Fc * NoL_Vis_PDF;
			}
		}

		{
			float3 L = CosineSampleHemisphere( E ).xyz;
			float3 H = normalize(V + L);

			float NoL = saturate( L.z );
			float NoH = saturate( H.z );
			float VoH = saturate( dot( V, H ) );

			float FD90 = ( 0.5 + 2 * VoH * VoH ) * Roughness;
			float FdV = 1 + (FD90 - 1) * pow( 1 - NoV, 5 );
			float FdL = 1 + (FD90 - 1) * pow( 1 - NoL, 5 );
			C += FdV * FdL * ( 1 - 0.3333 * Roughness );
		}
	}

	return float3( A, B, C ) / NumSamples;
}

float3 ApproximateSpecularIBL( uint2 Random, float3 SpecularColor, float Roughness, float3 N, float3 V )
{
	// Function replaced with prefiltered environment map sample
	float3 R = 2 * dot( V, N ) * N - V;
	float3 PrefilteredColor = PrefilterEnvMap( Random, Roughness, R );
	//float3 PrefilteredColor = FilterEnvMap( Random, Roughness, N, V );

	// Function replaced with 2D texture sample
	float NoV = saturate( dot( N, V ) );
	float2 AB = IntegrateBRDF( Random, Roughness, NoV ).xy;

	return PrefilteredColor * ( SpecularColor * AB.x + AB.y );
}


float3 ImageBasedLightingHair( FGBufferData GBuffer, float3 V, float3 N, uint2 Random )
{
	float3 Lighting = 0;

	uint NumSamples = 32;
	
	FHairTransmittanceData HairTransmittance = InitHairTransmittanceData(true);

	LOOP
	for( uint i = 0; i < NumSamples; i++ )
	{
		float2 E = Hammersley( i, NumSamples, Random );
		float3 L = UniformSampleSphere( E ).xyz;
		
		{
			float3 SampleColor = AmbientCubemap.SampleLevel( AmbientCubemapSampler, L, 0 ).rgb;

			float PDF = 1.0 / (4 * PI);

			float InvWeight = PDF * NumSamples;
			float Weight = rcp( InvWeight );

			float3 Shading = 0;
			Shading = HairShading( GBuffer, L, V, N, 1, HairTransmittance, 0, 0, Random );
			
			Lighting += SampleColor * Shading * Weight;
		}
	}

	return Lighting;
}

void MainPS(
	in noperspective float4 UVAndScreenPos : TEXCOORD0, 
	in FStereoPSInput StereoInput,
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
    , out float OutSeparatedSubsurfaceDiffuse : SV_Target1
#endif
)
{
	StereoSetupPS(StereoInput);

	float2 BufferUV = UVAndScreenPos.xy;
	float2 ScreenSpacePos = UVAndScreenPos.zw;
	int2 PixelPos = int2(SvPosition.xy);

	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);

	float AbsoluteDiffuseMip = AmbientCubemapMipAdjust.z;


	float3 ScreenVector = normalize(mul(float3(ScreenSpacePos, 1), DFToFloat3x3(PrimaryView.ScreenToWorld)).xyz);

	float3 N = GBuffer.WorldNormal;
	float3 V = -ScreenVector;
	float3 R0 = 2 * dot( V, N ) * N - V;

	//float NoV = max( dot(N, V), 1e-5 );
	float NoV = saturate( dot(N, V) );

	// Point lobe in off-specular peak direction
	float a = Square( GBuffer.Roughness );
	float3 R = lerp( N, R0, (1 - a) * ( sqrt(1 - a) + a ) );
	
	uint2 Random = Rand3DPCG16( uint3( PixelPos, View.StateFrameIndexMod8) ).xy;

	float3 NonSpecularContribution = 0;
	float3 SpecularContribution = 0;
	
	BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
	{
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0/255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
		N = OctahedronToUnitVector(oct1);			
	}
	float3 DiffuseLookup =  TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, N, AbsoluteDiffuseMip).rgb;
	NonSpecularContribution += GBuffer.DiffuseColor * DiffuseLookup;

	// Diffuse
	{
		// we want to access the mip with the preconvolved diffuse lighting (coneangle=90 degree)
		//NonSpecularContribution += GBuffer.DiffuseColor * DiffuseLookup;
	}

	// Specular
	{

		BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT )
		{
			float ClearCoat			  = GBuffer.CustomData.x;

			#if CLEAR_COAT_BOTTOM_NORMAL
				//const float3 ClearCoatUnderNormal = OctahedronToUnitVector((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 2) - (256.0/255.0));
				const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0/255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
 				const float3 ClearCoatUnderNormal = OctahedronToUnitVector(oct1);
				float3 R2 = 2 * dot( V, ClearCoatUnderNormal ) * ClearCoatUnderNormal - V;
			
				float Mip = ComputeCubemapMipFromRoughness( GBuffer.Roughness, AmbientCubemapMipAdjust.w );
				float3 SampleColor = TextureCubeSampleLevel( AmbientCubemap, AmbientCubemapSampler, R2, Mip ).rgb;
			
				float2 AB = PreIntegratedGF.SampleLevel( PreIntegratedGFSampler, float2( NoV, GBuffer.Roughness ), 0 ).rg;
				SpecularContribution += SampleColor * ( GBuffer.SpecularColor * AB.x + AB.y * (1 - ClearCoat) );
			#else
				float Mip = ComputeCubemapMipFromRoughness( GBuffer.Roughness, AmbientCubemapMipAdjust.w );
				float3 SampleColor = TextureCubeSampleLevel( AmbientCubemap, AmbientCubemapSampler, R, Mip ).rgb;

				float2 AB = PreIntegratedGF.SampleLevel( PreIntegratedGFSampler, float2( NoV, GBuffer.Roughness ), 0 ).rg;
				SpecularContribution += SampleColor * ( GBuffer.SpecularColor * AB.x + AB.y * (1 - ClearCoat) );
			#endif
		}
		else
		{
			float Mip = ComputeCubemapMipFromRoughness( GBuffer.Roughness, AmbientCubemapMipAdjust.w );
			float3 SampleColor = TextureCubeSampleLevel( AmbientCubemap, AmbientCubemapSampler, R, Mip ).rgb;

			SpecularContribution += SampleColor * EnvBRDF( GBuffer.SpecularColor, GBuffer.Roughness, NoV );
			//SpecularContribution += ApproximateSpecularIBL( Random, GBuffer.SpecularColor, GBuffer.Roughness, GBuffer.WorldNormal, -ScreenVector );
		}
	}

	BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT )
	{
		const float ClearCoat			= GBuffer.CustomData.x;
		const float ClearCoatRoughness	= GBuffer.CustomData.y;

		float Mip = ComputeCubemapMipFromRoughness( ClearCoatRoughness, AmbientCubemapMipAdjust.w );
		float3 SampleColor = TextureCubeSampleLevel( AmbientCubemap, AmbientCubemapSampler, R0, Mip ).rgb;

		// F_Schlick
		float F0 = 0.04;
		float Fc = pow( 1 - NoV, 5 );
		float F = Fc + (1 - Fc) * F0;
		F *= ClearCoat;

		float LayerAttenuation = (1 - F);

		NonSpecularContribution *= LayerAttenuation;
		SpecularContribution *= LayerAttenuation;
		SpecularContribution += SampleColor * F;
	}
	
	BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_HAIR )
	{
		FHairTransmittanceData HairTransmittance = InitHairTransmittanceData(true);
		float3 FakeNormal = normalize( V - N * dot(V,N) );
		FakeNormal = normalize( FakeNormal + 0 * float3(0,0,1) );
		SpecularContribution = TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, FakeNormal, AbsoluteDiffuseMip).rgb;
		SpecularContribution *= PI * HairShading( GBuffer, FakeNormal, V, N, 1, HairTransmittance, 0, 0.2, Random);
		NonSpecularContribution = 0;
	}

#if IMPORTANCE_SAMPLE
	if( GBuffer.ShadingModelID > 0 && BufferUV.x > 0.5 )
	{
		NonSpecularContribution = 0;

		BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_HAIR )
		{
			SpecularContribution = ImageBasedLightingHair( GBuffer, -ScreenVector, GBuffer.WorldNormal, Random );
		}
		else
		{
			SpecularContribution = ImageBasedLightingMIS( GBuffer, -ScreenVector, GBuffer.WorldNormal, Random );
		}
	}
#endif

	// apply darkening from ambient occlusion (does not use PostprocessInput1 to set white texture if SSAO is off)
	float AmbientOcclusion = GBuffer.GBufferAO * AmbientOcclusionTexture.SampleLevel(AmbientOcclusionSampler, BufferUV, 0).r;

	// Subsurface
	BRANCH if(GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		// some view dependent and some non view dependent (hard coded)
		float DependentSplit = 0.5f;

		float3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);

		// view independent (shared lookup for diffuse for better performance
		NonSpecularContribution += DiffuseLookup * SubsurfaceColor * (DependentSplit);
		// view dependent (blurriness is hard coded)
		SpecularContribution += TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, ScreenVector, AbsoluteDiffuseMip - 2.5f).rgb * SubsurfaceColor * (AmbientOcclusion * (1.0f - DependentSplit));
	}

	FLightAccumulator LightAccumulator = (FLightAccumulator)0;

	const bool bNeedsSeparateSubsurfaceLightAccumulation = UseSubsurfaceProfile(GBuffer.ShadingModelID);

	LightAccumulator_Add(LightAccumulator, NonSpecularContribution + SpecularContribution, NonSpecularContribution, AmbientCubemapColor.rgb, bNeedsSeparateSubsurfaceLightAccumulation);

	OutColor = LightAccumulator_GetResult(LightAccumulator);

	OutColor *= AmbientOcclusion;
	
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif
}
#endif // SUBSTRATE_ENABLED