// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RectLight.ush"

struct FAreaLight
{
	float		SphereSinAlpha;
	float		SphereSinAlphaSoft;
	float		LineCosSubtended;

	half3		DiffuseL;	// Vector irradiance
	half3		SpecularL;	// Closest to reflection vector
	half		NoL;
	float		Falloff;
	half3		FalloffColor;

	FRect		Rect;
	FRectTexture Texture;

	uint		IsRectAndDiffuseMicroReflWeight;
};

void SetIsRectLight(inout FAreaLight AreaLight, bool bIsRectLight)
{
	AreaLight.IsRectAndDiffuseMicroReflWeight = (AreaLight.IsRectAndDiffuseMicroReflWeight & 0xFFFFFFFE) | (bIsRectLight ? 0x1 : 0x0);
}

bool IsRectLight(FAreaLight AreaLight)
{
	return (AreaLight.IsRectAndDiffuseMicroReflWeight & 0x00000001) == 0x1;
}

void SetAreaLightDiffuseMicroReflWeight(inout FAreaLight AreaLight, float Weight)
{
	// We store the 32 bits float as 31 bits by droping the sign bit.
	AreaLight.IsRectAndDiffuseMicroReflWeight = (AreaLight.IsRectAndDiffuseMicroReflWeight & 0x00000001) | (asuint(Weight) << 1);
}

float GetAreaLightDiffuseMicroReflWeight(FAreaLight AreaLight)
{
	return asfloat(AreaLight.IsRectAndDiffuseMicroReflWeight >> 1);
}

bool IsAreaLight(FAreaLight AreaLight)
{
	return IsRectLight(AreaLight) || GetAreaLightDiffuseMicroReflWeight(AreaLight) < 1.0f;
}

FAreaLight InitAreaLight()
{
	FAreaLight AreaLight;
	AreaLight.SphereSinAlpha = 0;
	AreaLight.SphereSinAlphaSoft = 0;
	AreaLight.LineCosSubtended = 1;
	AreaLight.DiffuseL = 0;
	AreaLight.SpecularL = 0;
	AreaLight.NoL = 1;
	AreaLight.Falloff = 1;
	AreaLight.FalloffColor = 1;
	AreaLight.Rect = (FRect)0;
	AreaLight.IsRectAndDiffuseMicroReflWeight = 0;
	AreaLight.Texture = InitRectTexture();
	return AreaLight;
}
