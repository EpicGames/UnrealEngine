// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AssertionMacros.ush: Macros for Assertion
=============================================================================*/

#pragma once

#include "/Engine/Public/Platform.ush"

#if !defined(DO_CHECK)
	#define DO_CHECK 0
#endif 

#if !defined(DO_GUARD_SLOW)
	#define DO_GUARD_SLOW 0
#endif 


#if DO_CHECK

// Allow platform-specific overrides?
#if !defined(check)
	/**
	 * Check macro, intended to detect serious, execution-disrupting (not continuable) errors in the shader code.
	 * Is controlled by the read-only CVar `r.Shaders.CheckLevel`:
	 *		0           => DO_CHECK=0 and DO_GUARD_SLOW=0
	 *		1 (default) => DO_CHECK=1 and DO_GUARD_SLOW=0
	 *		2           => DO_CHECK=1 and DO_GUARD_SLOW=1
	 * NOTE: As check() is enabled by default this will cause increased register pressure and have some overhead. Therefore, be careful when using this
	 * and if in doubt, default to using checkSlow et al.
	 * When not doing GPU performance profiling it may be a good idea to keep r.Shaders.CheckLevel=2 to catch errors early.
	 */
	#define check(expr)	 \
		{ \
			if (!(expr)) \
			{ \
				PLATFORM_BREAK(); \
			} \
		}
#endif // check


	#define checkBufferAccess(buffer, index)	\
	{	\
		uint BufferCheckNumElements = 0U;	\
		buffer.GetDimensions(BufferCheckNumElements);	\
		check(uint(index) < BufferCheckNumElements);	\
	}


	#define checkStructuredBufferAccess(buffer, index)	\
	{	\
		uint BufferCheckNumElements = 0U;	\
		uint BufferCheckStride = 0U;	\
		buffer.GetDimensions(BufferCheckNumElements, BufferCheckStride);	\
		check(uint(index) < BufferCheckNumElements);	\
	}

#else // !DO_CHECK
	void CheckNop() {}

	#define check(expr) CheckNop()

	#define checkBufferAccess(buffer, index) CheckNop()
	#define checkStructuredBufferAccess(buffer, index) CheckNop()
#endif // DO_CHECK

#if DO_GUARD_SLOW
	#define checkSlow(expr)					check(expr)

	#define checkBufferAccessSlow(buffer, index) checkBufferAccess(buffer, index)
	#define checkStructuredBufferAccessSlow(buffer, index)	checkStructuredBufferAccess(buffer, index)

#else // DO_GUARD_SLOW
	void GuardSlowNop() {}

	#define checkSlow(expr) GuardSlowNop()

	#define checkBufferAccessSlow(buffer, index) GuardSlowNop()
	#define checkStructuredBufferAccessSlow(buffer, index) GuardSlowNop()
#endif // DO_GUARD_SLOW
