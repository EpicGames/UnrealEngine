// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BCCompressionCommon.ush:
	Helpers for compute shader BC texture compression
	
	todo[vt]: First implementation is keeping it simple. Lots of possible optimizations to do...
	* Pack float3 colors to uint earlier to reduce vector register pressure?
	* Calling code can build blocks with one color sample per thread (with layout optimized for coalesce)?
	* Code here could use wave ops for the block processing?
=============================================================================*/

#pragma once

#include "GammaCorrectionCommon.ush"

// A bit slower, but higher quality compression
#define LEAST_SQUARES_ENDPOINT_OPTIMIZATION 1

uint RoundToUInt(float X)
{
	return (uint)round(X);
}

// Simple convert float3 color to 565 uint using 'round' arithmetic
uint Float3ToUint565( in float3 Color )
{
	float3 Scale       = float3(31.f, 63.f, 31.f);
	float3 ColorScaled = round(saturate(Color) * Scale);
	uint ColorPacked   = ((uint)ColorScaled.r << 11) | ((uint)ColorScaled.g << 5) | (uint)ColorScaled.b;
	
	return ColorPacked;
}

// Convert float3 color to 565 uint using 'ceil' arithmetic
// Color parameter is inout and is modified to match the converted value
uint Float3ToUint565_Ceil( inout float3 Color )
{
	float3 Scale         = float3(31.f, 63.f, 31.f);
	float3 ColorScaled   = ceil(saturate(Color) * Scale);
	uint ColorPacked     = ((uint)ColorScaled.r << 11) | ((uint)ColorScaled.g << 5) | (uint)ColorScaled.b;
	Color				 = ColorScaled / Scale;

	return ColorPacked;
}

// Convert float3 color to 565 uint using 'floor' arithmetic
// Color parameter is inout and is modified to match the converted value
uint Float3ToUint565_Floor( inout float3 Color )
{
	float3 Scale         = float3(31.f, 63.f, 31.f);
	float3 ColorScaled   = floor(saturate(Color) * Scale);
	uint ColorPacked     = ((uint)ColorScaled.r << 11) | ((uint)ColorScaled.g << 5) | (uint)ColorScaled.b;
	Color				 = ColorScaled / Scale;

	return ColorPacked;
}

// Get min and max values in a single channel block
void GetMinMax( in float Block[16], out float OutMin, out float OutMax )
{
	OutMin = Block[0];
	OutMax = Block[0];

	for (int i=1; i<16; ++i)
	{
		OutMin = min(OutMin, Block[i]);
		OutMax = max(OutMax, Block[i]);
	}
}

// Get min and max values in two single channel blocks
void GetMinMax( in float Block0[16], in float Block1[16], out float OutMin0, out float OutMax0, out float OutMin1, out float OutMax1 )
{
	OutMin0 = Block0[0];
	OutMax0 = Block0[0];
	OutMin1 = Block1[0];
	OutMax1 = Block1[0];

	for (int i=1; i<16; ++i)
	{
		OutMin0 = min(OutMin0, Block0[i]);
		OutMax0 = max(OutMax0, Block0[i]);
		OutMin1 = min(OutMin1, Block1[i]);
		OutMax1 = max(OutMax1, Block1[i]);
	}
}

// Get min and max values in a single three channel block
void GetMinMax( in float3 Block[16], out float3 OutMin, out float3 OutMax )
{
	OutMin = Block[0];
	OutMax = Block[0];

	for (int i=1; i<16; ++i)
	{
		OutMin = min(OutMin, Block[i]);
		OutMax = max(OutMax, Block[i]);
	}
}

// Calculate the final packed indices for a color block
uint GetPackedColorBlockIndices( in float3 Block[16], in float3 MinColor, in float3 MaxColor )
{
	uint PackedIndices = 0;

	// Project onto max->min color vector and segment into range [0,3]
	float3 Range = MinColor - MaxColor;
	float Scale = 3.f / dot(Range, Range);
	float3 ScaledRange  = Range * Scale;
	float Bias = (dot(MaxColor, MaxColor) - dot(MaxColor, MinColor)) * Scale;
	
	for (int i=15; i>=0; --i)
	{
		// Compute the distance index for this element
		uint Index = RoundToUInt(dot(Block[i], ScaledRange) + Bias);
		// Convert distance index into the BC index
		Index += (Index > 0) - (3 * (Index == 3));
		// OR into the final PackedIndices
		PackedIndices = (PackedIndices<<2)|Index;
	}

	return PackedIndices;
}

// Calculate the final packed indices for an alpha block
// The results are in their final location of the uint2 indices and will need ORing with the min and max alpha
uint2 GetPackedAlphaBlockIndices( in float Block[16], in float MinAlpha, in float MaxAlpha )
{
	uint2 PackedIndices = 0;

	// Segment alpha max->min into range [0,7]
	float Range = MinAlpha - MaxAlpha;
	float Scale = 7.f / Range;
	float Bias = -Scale * MaxAlpha;

	uint i = 0;
	// The first 5 elements of the block will go into the top 16 bits of the x component
	for (i=0; i<5; ++i)
	{
		// Compute the distance index for this element
		uint Index = RoundToUInt(Block[i] * Scale + Bias);
		// Convert distance index into the BC index
		Index += (Index > 0) - (7 * (Index == 7));
		// OR into the final PackedIndices
		PackedIndices.x |= (Index << (i*3 + 16));
	}

	// The 6th element is split across the x and y components
	{
		i = 5;
		uint Index = RoundToUInt(Block[i] * Scale + Bias);
		Index += (Index > 0) - (7 * (Index == 7));
		PackedIndices.x |= (Index << 31);
		PackedIndices.y |= (Index >> 1);
	}

	// The rest of the elements fill the y component
	for (i=6; i<16; ++i)
	{
		uint Index = RoundToUInt(Block[i] * Scale + Bias);
		Index += (Index > 0) - (7 * (Index == 7 ? 1 : 0));
		PackedIndices.y |= (Index << (i*3 - 16));
	}

	return PackedIndices;
}

// Compress a BC1 block
uint2 CompressBC1Block( in float3 Block[16] )
{
	float3 MinColor, MaxColor;
	GetMinMax( Block, MinColor, MaxColor );

	uint MinColor565 = Float3ToUint565_Floor( MinColor );
	uint MaxColor565 = Float3ToUint565_Ceil( MaxColor );

	uint Indices = 0;
	if (MinColor565 < MaxColor565)
	{
		Indices = GetPackedColorBlockIndices( Block, MinColor, MaxColor );
	}

	return uint2((MinColor565 << 16) | MaxColor565, Indices);
}

// Compress a BC1 block that will be sampled as sRGB
// We expect linear colors as input and convert internally
uint2 CompressBC1Block_SRGB( in float3 Block[16] )
{
	float3 MinColor, MaxColor;
	GetMinMax( Block, MinColor, MaxColor );

	uint MinColor565 = Float3ToUint565( LinearToSrgb(MinColor) );
	uint MaxColor565 = Float3ToUint565( LinearToSrgb(MaxColor) );

	uint Indices = 0;
	if (MinColor565 < MaxColor565)
	{
		Indices = GetPackedColorBlockIndices( Block, MinColor, MaxColor );
	}

	return uint2((MinColor565 << 16) | MaxColor565, Indices);
}

// Compress a BC3 block
uint4 CompressBC3Block( in float3 BlockRGB[16], in float BlockA[16] )
{
	float3 MinColor, MaxColor;
	GetMinMax( BlockRGB, MinColor, MaxColor );

	uint MinColor565 = Float3ToUint565_Floor( MinColor );
	uint MaxColor565 = Float3ToUint565_Ceil( MaxColor );

	float MinAlpha, MaxAlpha;
	GetMinMax( BlockA, MinAlpha, MaxAlpha );

	uint MinAlphaUint = RoundToUInt(MinAlpha * 255.f);
	uint MaxAlphaUint = RoundToUInt(MaxAlpha * 255.f);

	uint ColorIndices = 0;
	if (MinColor565 < MaxColor565)
	{
		ColorIndices = GetPackedColorBlockIndices( BlockRGB, MinColor, MaxColor );
	}

	uint2 AlphaIndices = 0;
	if (MinAlphaUint < MaxAlphaUint)
	{
		AlphaIndices = GetPackedAlphaBlockIndices(BlockA, MinAlpha, MaxAlpha);
	}

	return uint4((MinAlphaUint << 8) | MaxAlphaUint | AlphaIndices.x, AlphaIndices.y, (MinColor565 << 16) | MaxColor565, ColorIndices);
}

// Compress a BC3 block that will be sampled as sRGB
// We expect linear colors as input and convert internally
uint4 CompressBC3Block_SRGB( in float3 BlockRGB[16], in float BlockA[16] )
{
	float3 MinColor, MaxColor;
	GetMinMax( BlockRGB, MinColor, MaxColor );

	uint MinColor565 = Float3ToUint565( LinearToSrgb( MinColor ) );
	uint MaxColor565 = Float3ToUint565( LinearToSrgb( MaxColor ) );

	float MinAlpha, MaxAlpha;
	GetMinMax( BlockA, MinAlpha, MaxAlpha );

	uint MinAlphaUint = RoundToUInt(MinAlpha * 255.f);
	uint MaxAlphaUint = RoundToUInt(MaxAlpha * 255.f);

	uint ColorIndices = 0;
	if (MinColor565 < MaxColor565)
	{
		ColorIndices = GetPackedColorBlockIndices(BlockRGB, MinColor, MaxColor);
	}

	uint2 AlphaIndices = 0;
	if (MinAlphaUint < MaxAlphaUint)
	{
		AlphaIndices = GetPackedAlphaBlockIndices(BlockA, MinAlpha, MaxAlpha);
	}

	return uint4((MinAlphaUint << 8) | MaxAlphaUint | AlphaIndices.x, AlphaIndices.y, (MinColor565 << 16) | MaxColor565, ColorIndices);
}

// Compress a BC4 block
uint2 CompressBC4Block( in float Block[16] )
{
	float MinAlpha, MaxAlpha;
	GetMinMax( Block, MinAlpha, MaxAlpha );

	uint MinAlphaUint = RoundToUInt(MinAlpha * 255.f);
	uint MaxAlphaUint = RoundToUInt(MaxAlpha * 255.f);

	uint2 Indices = 0;
	if (MinAlphaUint < MaxAlphaUint)
	{
		Indices = GetPackedAlphaBlockIndices( Block, MinAlpha, MaxAlpha );
	}

	uint2 BC4Block = 0;
	BC4Block.x = (MinAlphaUint << 8) | MaxAlphaUint;
	BC4Block.x |= Indices.x;
	BC4Block.y = Indices.y;
	return BC4Block;
}

// Compress a BC5 block
uint4 CompressBC5Block( in float BlockU[16], in float BlockV[16] )
{
	float MinU, MaxU, MinV, MaxV;
	GetMinMax( BlockU, BlockV, MinU, MaxU, MinV, MaxV );

	uint MinUUint = RoundToUInt(MinU * 255.f);
	uint MaxUUint = RoundToUInt(MaxU * 255.f);
	uint MinVUint = RoundToUInt(MinV * 255.f);
	uint MaxVUint = RoundToUInt(MaxV * 255.f);

	uint2 IndicesU = 0;
	if (MinUUint < MaxUUint)
	{
		IndicesU = GetPackedAlphaBlockIndices( BlockU, MinU, MaxU );
	}

	uint2 IndicesV = 0;
	if (MinVUint < MaxVUint)
	{
		IndicesV = GetPackedAlphaBlockIndices( BlockV, MinV, MaxV );
	}

	return uint4((MinUUint << 8) | MaxUUint | IndicesU.x, IndicesU.y, (MinVUint << 8) | MaxVUint | IndicesV.x, IndicesV.y);
}

// Convert RGB linear color to YCoCG
float3 RGB2YCoCg(float3 RGB)
{
	float Y = (RGB.r + 2.f * RGB.g + RGB.b) * 0.25f;
	float Co = ((2.f * RGB.r - 2.f * RGB.b) * 0.25f + 128.f / 255.f);
	float Cg = ((-RGB.r + 2.f * RGB.g - RGB.b) * 0.25f + 128.f / 255.f);
	return float3(Y, Co, Cg);
}

// Get a single scale factor to use for a YCoCg color block
// This increases precision at the expense of potential blending artifacts across blocks
float2 GetYCoCgScale(float2 MinCoCg, float2 MaxCoCg)
{
	MinCoCg = abs(MinCoCg - 128.f / 255.f);
	MaxCoCg = abs(MaxCoCg - 128.f / 255.f);

	float MaxComponent = max(max(MinCoCg.x, MinCoCg.y), max(MaxCoCg.x, MaxCoCg.y));

	return (MaxComponent < 32.f / 255.f) ? float2(4.f, 0.25f) : (MaxComponent < 64.f / 255.f) ? float2(2.f, 0.5f) : float2(1.f, 1.f);
}

void ApplyYCoCgScale(inout float2 MinCoCg, inout float2 MaxCoCg, float Scale)
{
	MinCoCg = (MinCoCg - 128.f / 255.f) * Scale + 128.f / 255.f;
	MaxCoCg = (MaxCoCg - 128.f / 255.f) * Scale + 128.f / 255.f;
}

// Inset the CoCg bounding end points
void InsetCoCgEndPoints(inout float2 MinCoCg, inout float2 MaxCoGg)
{
	float2 Inset = (MaxCoGg - MinCoCg - (8.f / 255.f)) / 16.f;
	MinCoCg = saturate(MinCoCg + Inset);
	MaxCoGg = saturate(MaxCoGg - Inset);
}

// Inset the luminance end points
void InsetLumaEndPoints(inout float MinY, inout float MaxY)
{
	float Inset = (MaxY - MinY - (16.f / 255.f)) / 32.f;
	MinY = saturate(MinY + Inset);
	MaxY = saturate(MaxY - Inset);
}

// Select the 2 min/max end points from the CoCg bounding rectangle based on the block contents 
void AdjustMinMaxDiagonalYCoCg(const float3 Block[16], inout float2 MinCoCg, inout float2 MaxCoGg)
{
	float2 MidCoCg = (MaxCoGg + MinCoCg) * 0.5;

	float Sum = 0.f;
	for (int i = 0; i < 16; i++)
	{
		float2 Diff = Block[i].yz - MidCoCg;
		Sum += Diff.x * Diff.y;
	}
	if (Sum < 0.f)
	{
		float Temp = MaxCoGg.y;
		MaxCoGg.y = MinCoCg.y;
		MinCoCg.y = Temp;
	}
}

uint CoCgToUint565(inout float2 CoCg)
{
	float2 Scale = float2(31.f, 63.f);
	float2 ColorScaled = round(saturate(CoCg) * Scale);
	uint ColorPacked = ((uint)ColorScaled.r << 11) | ((uint)ColorScaled.g << 5);
	CoCg = ColorScaled / Scale;

	return ColorPacked;
}

// Calculate the final packed indices for the CoCg part of a color block
uint GetPackedCoCgBlockIndices(in float3 Block[16], in float2 MinCoCg, in float2 MaxCoCg)
{
	uint PackedIndices = 0;

	// Project onto max->min color vector and segment into range [0,3]
	float2 Range = MinCoCg - MaxCoCg;
	float Scale = 3.f / dot(Range, Range);
	float2 ScaledRange = Range * Scale;
	float Bias = (dot(MaxCoCg, MaxCoCg) - dot(MaxCoCg, MinCoCg)) * Scale;

	for (int i = 15; i >= 0; --i)
	{
		// Compute the distance index for this element
		uint Index = RoundToUInt(dot(Block[i].yz, ScaledRange) + Bias);
		// Convert distance index into the BC index
		Index += (Index > 0) - (3 * (Index == 3));
		// OR into the final PackedIndices
		PackedIndices = (PackedIndices << 2) | Index;
	}

	return PackedIndices;
}

// Calculate the final packed indices for the Luma part of a color block
uint2 GetPackedLumaBlockIndices(in float3 Block[16], in float MinAlpha, in float MaxAlpha)
{
	uint2 PackedIndices = 0;

	// Segment alpha max->min into range [0,7]
	float Range = MinAlpha - MaxAlpha;
	float Scale = 7.f / Range;
	float Bias = -Scale * MaxAlpha;

	uint i = 0;
	// The first 5 elements of the block will go into the top 16 bits of the x component
	for (i = 0; i < 5; ++i)
	{
		// Compute the distance index for this element
		uint Index = RoundToUInt(Block[i].x * Scale + Bias);
		// Convert distance index into the BC index
		Index += (Index > 0) - (7 * (Index == 7));
		// OR into the final PackedIndices
		PackedIndices.x |= (Index << (i * 3 + 16));
	}

	// The 6th element is split across the x and y components
	{
		i = 5;
		uint Index = RoundToUInt(Block[i].x * Scale + Bias);
		Index += (Index > 0) - (7 * (Index == 7));
		PackedIndices.x |= (Index << 31);
		PackedIndices.y |= (Index >> 1);
	}

	// The rest of the elements fill the y component
	for (i = 6; i < 16; ++i)
	{
		uint Index = RoundToUInt(Block[i].x * Scale + Bias);
		Index += (Index > 0) - (7 * (Index == 7 ? 1 : 0));
		PackedIndices.y |= (Index << (i * 3 - 16));
	}

	return PackedIndices;
}

// Convert a linear RGB block to YCoCg and compress a BC3 block 
uint4 CompressBC3BlockYCoCg(in float3 Block[16])
{
	for (int i = 0; i < 16; ++i)
	{
		Block[i] = RGB2YCoCg(Block[i]);
	}

	float3 MinColor, MaxColor;
	GetMinMax(Block, MinColor, MaxColor);

	AdjustMinMaxDiagonalYCoCg(Block, MinColor.yz, MaxColor.yz);

	float2 Scale = GetYCoCgScale(MinColor.yz, MaxColor.yz);

	ApplyYCoCgScale(MinColor.yz, MaxColor.yz, Scale.x);

	InsetCoCgEndPoints(MinColor.yz, MaxColor.yz);

	uint MinColor565 = CoCgToUint565(MinColor.yz) | ((uint)Scale.x - 1);
	uint MaxColor565 = CoCgToUint565(MaxColor.yz) | ((uint)Scale.x - 1);

	ApplyYCoCgScale(MinColor.yz, MaxColor.yz, Scale.y);

	uint CoCgIndices = GetPackedCoCgBlockIndices(Block, MinColor.yz, MaxColor.yz);

	InsetLumaEndPoints(MinColor.x, MaxColor.x);

	uint MinLumaUint = RoundToUInt(MinColor.x * 255.f);
	uint MaxLumaUint = RoundToUInt(MaxColor.x * 255.f);

	uint2 Indices = GetPackedLumaBlockIndices(Block, MinColor.x, MaxColor.x);

	return uint4((MinLumaUint << 8) | MaxLumaUint | Indices.x, Indices.y, (MinColor565 << 16) | MaxColor565, CoCgIndices);
}

float3 Quantize10(float3 X)
{
	return (f32tof16(X) * 1024.0f) / (0x7bff + 1.0f);
}

uint ComputeIndexBC6HIndex(float3 Color, float3 BlockVector, float EndPoint0Pos, float EndPoint1Pos)
{
	float Pos = (float)f32tof16(dot(Color, BlockVector));
	float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
	return (uint)clamp(NormalizedPos * 14.93333f + 0.03333f + 0.5f, 0.0f, 15.0f);
}

// Compress a BC6H block. Evaluates only mode 11 for performance
uint4 CompressBC6HBlock(in float3 BlockRGB[16])
{
	// Compute initial endpoints
	float3 BlockMin = BlockRGB[0];
	float3 BlockMax = BlockRGB[0];
	{
		for (uint TexelIndex = 1; TexelIndex < 16; ++TexelIndex)
		{
			BlockMin = min(BlockMin, BlockRGB[TexelIndex]);
			BlockMax = max(BlockMax, BlockRGB[TexelIndex]);
		}
	}

	float3 BlockVector = BlockMax - BlockMin;
	BlockVector = BlockVector / (BlockVector.x + BlockVector.y + BlockVector.z);

	float3 Endpoint0 = Quantize10(BlockMin);
	float3 Endpoint1 = Quantize10(BlockMax);
	float EndPoint0Pos = (float)f32tof16(dot(BlockMin, BlockVector));
	float EndPoint1Pos = (float)f32tof16(dot(BlockMax, BlockVector));

	// Check if endpoint swap is required
	uint FixupIndex = ComputeIndexBC6HIndex(BlockRGB[0], BlockVector, EndPoint0Pos, EndPoint1Pos);
	if (FixupIndex > 7)
	{
		Swap(EndPoint0Pos, EndPoint1Pos);
		Swap(Endpoint0, Endpoint1);
	}

	// Compute indices
	uint Indices[16] = { 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 };
	for (uint TexelIndex = 0; TexelIndex < 16; ++TexelIndex)
	{
		Indices[TexelIndex] = ComputeIndexBC6HIndex(BlockRGB[TexelIndex], BlockVector, EndPoint0Pos, EndPoint1Pos);
	}

	uint4 Block = 0;
	{
		// Encode mode 11 block
		Block.x = 0x03;

		// Encode endpoints
		Block.x |= (uint)Endpoint0.x << 5;
		Block.x |= (uint)Endpoint0.y << 15;
		Block.x |= (uint)Endpoint0.z << 25;
		Block.y |= (uint)Endpoint0.z >> 7;
		Block.y |= (uint)Endpoint1.x << 3;
		Block.y |= (uint)Endpoint1.y << 13;
		Block.y |= (uint)Endpoint1.z << 23;
		Block.z |= (uint)Endpoint1.z >> 9;

		// Encode indices
		Block.z |= Indices[0] << 1;
		Block.z |= Indices[1] << 4;
		Block.z |= Indices[2] << 8;
		Block.z |= Indices[3] << 12;
		Block.z |= Indices[4] << 16;
		Block.z |= Indices[5] << 20;
		Block.z |= Indices[6] << 24;
		Block.z |= Indices[7] << 28;
		Block.w |= Indices[8] << 0;
		Block.w |= Indices[9] << 4;
		Block.w |= Indices[10] << 8;
		Block.w |= Indices[11] << 12;
		Block.w |= Indices[12] << 16;
		Block.w |= Indices[13] << 20;
		Block.w |= Indices[14] << 24;
		Block.w |= Indices[15] << 28;
	}

	return Block;
}

uint3 Quantize7(float3 X)
{
	return (uint3(X * 0xFF)) >> 1;
}

uint ComputeBC7Index(float3 Color, float3 BlockVector, float EndPoint0Pos, float EndPoint1Pos)
{
	float Pos = dot(Color, BlockVector);
	float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
	return (uint)clamp(NormalizedPos * 14.93333f + 0.03333f + 0.5f, 0.0f, 15.0f);
}

// Least squares optimization to find best endpoints for the selected block indices in a mode 6 BC7 block
void OptimizeEndpointsBC7(float3 Texels[16], inout float3 BlockMin, inout float3 BlockMax)
{
	float3 BlockVector = BlockMax - BlockMin;

	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	float3 AlphaTexelSum = 0.0f;
	float3 BetaTexelSum = 0.0f;
	float AlphaBetaSum = 0.0f;
	float AlphaSqSum = 0.0f;
	float BetaSqSum = 0.0f;

	for (uint TexelIndex = 0; TexelIndex < 16; ++TexelIndex)
	{
		uint Index = ComputeBC7Index(Texels[TexelIndex], BlockVector, EndPoint0Pos, EndPoint1Pos);

		float Beta = saturate(Index / 15.0f);
		float Alpha = 1.0f - Beta;

		AlphaTexelSum += Alpha * Texels[TexelIndex];
		BetaTexelSum += Beta * Texels[TexelIndex];

		AlphaBetaSum += Alpha * Beta;

		AlphaSqSum += Alpha * Alpha;
		BetaSqSum += Beta * Beta;
	}

	float Det = AlphaSqSum * BetaSqSum - AlphaBetaSum * AlphaBetaSum;

	if (abs(Det) > 0.1f)
	{
		float RcpDet = rcp(Det);
		BlockMin = saturate(RcpDet * (AlphaTexelSum * BetaSqSum - BetaTexelSum * AlphaBetaSum));
		BlockMax = saturate(RcpDet * (BetaTexelSum * AlphaSqSum - AlphaTexelSum * AlphaBetaSum));
	}
}

// Compress a BC7 color only block. Evaluates only mode 6 for performance
uint4 CompressBC7Block(in float3 BlockRGB[16])
{
	// Compute initial endpoints
	float3 BlockMin = BlockRGB[0];
	float3 BlockMax = BlockRGB[0];
	{
		for (uint TexelIndex = 1; TexelIndex < 16; ++TexelIndex)
		{
			BlockMin = min(BlockMin, BlockRGB[TexelIndex]);
			BlockMax = max(BlockMax, BlockRGB[TexelIndex]);
		}
	}

	#if LEAST_SQUARES_ENDPOINT_OPTIMIZATION
	{
		OptimizeEndpointsBC7(BlockRGB, BlockMin, BlockMax);
	}
	#endif

	float3 BlockVector = BlockMax - BlockMin;

	uint3 Endpoint0 = Quantize7(BlockMin);
	uint3 Endpoint1 = Quantize7(BlockMax);
	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	// Check if endpoint swap is required
	uint FixupIndex = ComputeBC7Index(BlockRGB[0], BlockVector, EndPoint0Pos, EndPoint1Pos);
	if (FixupIndex > 7)
	{
		Swap(EndPoint0Pos, EndPoint1Pos);
		Swap(Endpoint0, Endpoint1);
	}

	// Compute indices
	uint Indices[16] = { 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 };
	for (uint TexelIndex = 0; TexelIndex < 16; ++TexelIndex)
	{
		Indices[TexelIndex] = ComputeBC7Index(BlockRGB[TexelIndex], BlockVector, EndPoint0Pos, EndPoint1Pos);
	}

	uint4 Block = 0;
	Block.x = 0;
	Block.y = 0;
	Block.z = 0;
	Block.w = 0;
	{
		// Encode mode 6 block
		Block.x = 0x40;

		// Encode endpoints
		Block.x |= Endpoint0.x << 7;
		Block.x |= Endpoint1.x << 14;
		Block.x |= Endpoint0.y << 21;
		Block.x |= Endpoint1.y << 28;
		Block.y |= Endpoint1.y >> 4;
		Block.y |= Endpoint0.z << 3;
		Block.y |= Endpoint1.z << 10;

		// Encode endpoint p-bit

		// Encode indices
		Block.z |= Indices[0] << 1;
		Block.z |= Indices[1] << 4;
		Block.z |= Indices[2] << 8;
		Block.z |= Indices[3] << 12;
		Block.z |= Indices[4] << 16;
		Block.z |= Indices[5] << 20;
		Block.z |= Indices[6] << 24;
		Block.z |= Indices[7] << 28;
		Block.w |= Indices[8] << 0;
		Block.w |= Indices[9] << 4;
		Block.w |= Indices[10] << 8;
		Block.w |= Indices[11] << 12;
		Block.w |= Indices[12] << 16;
		Block.w |= Indices[13] << 20;
		Block.w |= Indices[14] << 24;
		Block.w |= Indices[15] << 28;
	}

	return Block;
}

uint4 Quantize7A(float4 X)
{
	return (uint4(X * 0xFF)) >> 1;
}

uint ComputeBC7AIndex(float4 Color, float4 BlockVector, float EndPoint0Pos, float EndPoint1Pos)
{
	float Pos = dot(Color, BlockVector);
	float NormalizedPos = saturate((Pos - EndPoint0Pos) / (EndPoint1Pos - EndPoint0Pos));
	return (uint)clamp(NormalizedPos * 14.93333f + 0.03333f + 0.5f, 0.0f, 15.0f);
}

// Least squares optimization to find best endpoints for the selected block indices in a mode 6 BC7 block
void OptimizeEndpointsBC7A(float4 Texels[16], inout float4 BlockMin, inout float4 BlockMax)
{
	float4 BlockVector = BlockMax - BlockMin;

	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	float4 AlphaTexelSum = 0.0f;
	float4 BetaTexelSum = 0.0f;
	float AlphaBetaSum = 0.0f;
	float AlphaSqSum = 0.0f;
	float BetaSqSum = 0.0f;

	for (uint TexelIndex = 0; TexelIndex < 16; ++TexelIndex)
	{
		uint Index = ComputeBC7AIndex(Texels[TexelIndex], BlockVector, EndPoint0Pos, EndPoint1Pos);

		float Beta = saturate(Index / 15.0f);
		float Alpha = 1.0f - Beta;

		AlphaTexelSum += Alpha * Texels[TexelIndex];
		BetaTexelSum += Beta * Texels[TexelIndex];

		AlphaBetaSum += Alpha * Beta;

		AlphaSqSum += Alpha * Alpha;
		BetaSqSum += Beta * Beta;
	}

	float Det = AlphaSqSum * BetaSqSum - AlphaBetaSum * AlphaBetaSum;

	if (abs(Det) > 0.1f)
	{
		float RcpDet = rcp(Det);
		BlockMin = saturate(RcpDet * (AlphaTexelSum * BetaSqSum - BetaTexelSum * AlphaBetaSum));
		BlockMax = saturate(RcpDet * (BetaTexelSum * AlphaSqSum - AlphaTexelSum * AlphaBetaSum));
	}
}

// Compress a BC7 color only block. Evaluates only mode 6 for performance
uint4 CompressBC7ABlock(in float4 BlockRGBA[16])
{
	// Compute initial endpoints
	float4 BlockMin = BlockRGBA[0];
	float4 BlockMax = BlockRGBA[0];
	{
		for (uint TexelIndex = 1; TexelIndex < 16; ++TexelIndex)
		{
			BlockMin = min(BlockMin, BlockRGBA[TexelIndex]);
			BlockMax = max(BlockMax, BlockRGBA[TexelIndex]);
		}
	}

	#if LEAST_SQUARES_ENDPOINT_OPTIMIZATION
	{
		OptimizeEndpointsBC7A(BlockRGBA, BlockMin, BlockMax);
	}
	#endif

	float4 BlockVector = BlockMax - BlockMin;
	uint4 Endpoint0 = Quantize7A(BlockMin);
	uint4 Endpoint1 = Quantize7A(BlockMax);
	float EndPoint0Pos = dot(BlockMin, BlockVector);
	float EndPoint1Pos = dot(BlockMax, BlockVector);

	// Check if endpoint swap is required
	uint FixupIndex = ComputeBC7AIndex(BlockRGBA[0], BlockVector, EndPoint0Pos, EndPoint1Pos);
	if (FixupIndex > 7)
	{
		Swap(EndPoint0Pos, EndPoint1Pos);
		Swap(Endpoint0, Endpoint1);
	}

	// Compute indices
	uint Indices[16] = { 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 };
	for (uint TexelIndex = 0; TexelIndex < 16; ++TexelIndex)
	{
		Indices[TexelIndex] = ComputeBC7AIndex(BlockRGBA[TexelIndex], BlockVector, EndPoint0Pos, EndPoint1Pos);
	}

	uint4 Block = 0;
	Block.x = 0;
	Block.y = 0;
	Block.z = 0;
	Block.w = 0;
	{
		// Encode mode 6 block
		Block.x = 0x40;

		// Encode endpoints
		Block.x |= Endpoint0.x << 7;
		Block.x |= Endpoint1.x << 14;
		Block.x |= Endpoint0.y << 21;
		Block.x |= Endpoint1.y << 28;
		Block.y |= Endpoint1.y >> 4;
		Block.y |= Endpoint0.z << 3;
		Block.y |= Endpoint1.z << 10;
		Block.y |= Endpoint0.w << 17;
		Block.y |= Endpoint1.w << 24;

		// Encode endpoint p-bit

		// Encode indices
		Block.z |= Indices[0] << 1;
		Block.z |= Indices[1] << 4;
		Block.z |= Indices[2] << 8;
		Block.z |= Indices[3] << 12;
		Block.z |= Indices[4] << 16;
		Block.z |= Indices[5] << 20;
		Block.z |= Indices[6] << 24;
		Block.z |= Indices[7] << 28;
		Block.w |= Indices[8] << 0;
		Block.w |= Indices[9] << 4;
		Block.w |= Indices[10] << 8;
		Block.w |= Indices[11] << 12;
		Block.w |= Indices[12] << 16;
		Block.w |= Indices[13] << 20;
		Block.w |= Indices[14] << 24;
		Block.w |= Indices[15] << 28;
	}

	return Block;
}
