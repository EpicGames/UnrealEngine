// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BRDF.usf: Bidirectional reflectance distribution functions.
=============================================================================*/

#pragma once

struct BxDFContext
{
	half NoV;
	half NoL;
	half VoL;
	half NoH;
	half VoH;
	half XoV;
	half XoL;
	half XoH;
	half YoV;
	half YoL;
	half YoH;
};

void Init( inout BxDFContext Context, half3 N, half3 V, half3 L )
{
	Context.NoL = dot(N, L);
	Context.NoV = dot(N, V);
	Context.VoL = dot(V, L);
	float InvLenH = rsqrt( 2 + 2 * Context.VoL );
	Context.NoH = saturate( ( Context.NoL + Context.NoV ) * InvLenH );
	Context.VoH = saturate( InvLenH + InvLenH * Context.VoL );
	//NoL = saturate( NoL );
	//NoV = saturate( abs( NoV ) + 1e-5 );

	Context.XoV = 0.0f;
	Context.XoL = 0.0f;
	Context.XoH = 0.0f;
	Context.YoV = 0.0f;
	Context.YoL = 0.0f;
	Context.YoH = 0.0f;
}

void Init( inout BxDFContext Context, half3 N, half3 X, half3 Y, half3 V, half3 L )
{
	Context.NoL = dot(N, L);
	Context.NoV = dot(N, V);
	Context.VoL = dot(V, L);
	float InvLenH = rsqrt( 2 + 2 * Context.VoL );
	Context.NoH = saturate( ( Context.NoL + Context.NoV ) * InvLenH );
	Context.VoH = saturate( InvLenH + InvLenH * Context.VoL );
	//NoL = saturate( NoL );
	//NoV = saturate( abs( NoV ) + 1e-5 );

	Context.XoV = dot(X, V);
	Context.XoL = dot(X, L);
	Context.XoH = (Context.XoL + Context.XoV) * InvLenH;
	Context.YoV = dot(Y, V);
	Context.YoL = dot(Y, L);
	Context.YoH = (Context.YoL + Context.YoV) * InvLenH;
}

// [ de Carpentier 2017, "Decima Engine: Advances in Lighting and AA" ]
void SphereMaxNoH( inout BxDFContext Context, float SinAlpha, bool bNewtonIteration )
{
	if( SinAlpha > 0 )
	{
		float CosAlpha = sqrt( 1 - Pow2( SinAlpha ) );
	
		float RoL = 2 * Context.NoL * Context.NoV - Context.VoL;
		if( RoL >= CosAlpha )
		{
			Context.NoH = 1;
			Context.XoH = 0;
			Context.YoH = 0;
			Context.VoH = abs( Context.NoV );
		}
		else
		{
			float rInvLengthT = SinAlpha * rsqrt( 1 - RoL*RoL );
			float NoTr = rInvLengthT * ( Context.NoV - RoL * Context.NoL );
// Enable once anisotropic materials support area lights
#if 0
			float XoTr = rInvLengthT * ( Context.XoV - RoL * Context.XoL );
			float YoTr = rInvLengthT * ( Context.YoV - RoL * Context.YoL );
#endif
			float VoTr = rInvLengthT * ( 2 * Context.NoV*Context.NoV - 1 - RoL * Context.VoL );

			if (bNewtonIteration)
			{
				// dot( cross(N,L), V )
				float NxLoV = sqrt( saturate( 1 - Pow2(Context.NoL) - Pow2(Context.NoV) - Pow2(Context.VoL) + 2 * Context.NoL * Context.NoV * Context.VoL ) );

				float NoBr = rInvLengthT * NxLoV;
				float VoBr = rInvLengthT * NxLoV * 2 * Context.NoV;

				float NoLVTr = Context.NoL * CosAlpha + Context.NoV + NoTr;
				float VoLVTr = Context.VoL * CosAlpha + 1   + VoTr;

				float p = NoBr   * VoLVTr;
				float q = NoLVTr * VoLVTr;
				float s = VoBr   * NoLVTr;

				float xNum = q * ( -0.5 * p + 0.25 * VoBr * NoLVTr );
				float xDenom = p*p + s * (s - 2*p) + NoLVTr * ( (Context.NoL * CosAlpha + Context.NoV) * Pow2(VoLVTr) + q * (-0.5 * (VoLVTr + Context.VoL * CosAlpha) - 0.5) );
				float TwoX1 = 2 * xNum / ( Pow2(xDenom) + Pow2(xNum) );
				float SinTheta = TwoX1 * xDenom;
				float CosTheta = 1.0 - TwoX1 * xNum;
				NoTr = CosTheta * NoTr + SinTheta * NoBr;
				VoTr = CosTheta * VoTr + SinTheta * VoBr;
			}

			Context.NoL = Context.NoL * CosAlpha + NoTr; // dot( N, L * CosAlpha + T * SinAlpha )
// Enable once anisotropic materials support area lights
#if 0
			Context.XoL = Context.XoL * CosAlpha + XoTr;
			Context.YoL = Context.YoL * CosAlpha + YoTr;
#endif
			Context.VoL = Context.VoL * CosAlpha + VoTr;

#if SUBSTRATE_ENABLED
			// It seems that VoL can end up out of the [-1, 1] range.
			// This causes NaN when evaluating the phase function for Substrate SimpleVolume diffuse (sqrt of a negative number is NaN, see HenyeyGreensteinPhase).
			// So we enforce a valid VoL range here for substrate which does do math when VoL is close to -1 in this case.
			Context.VoL = clamp(Context.VoL, -1.0, 1.0);
#endif

			float InvLenH = rsqrt( 2 + 2 * Context.VoL );
			Context.NoH = saturate( ( Context.NoL + Context.NoV ) * InvLenH );
// Enable once anisotropic materials support area lights
#if 0
			Context.XoH = ((Context.XoL + Context.XoV) * InvLenH);	// dot(X, (L+V)/|L+V|)
			Context.YoH = ((Context.YoL + Context.YoV) * InvLenH);
#endif
			Context.VoH = saturate( InvLenH + InvLenH * Context.VoL );
		}
	}
}

// Physically based shading model
// parameterized with the below options
// [ Karis 2013, "Real Shading in Unreal Engine 4" slide 11 ]

// E = Random sample for BRDF.
// N = Normal of the macro surface.
// H = Normal of the micro surface.
// V = View vector going from surface's position towards the view's origin.
// L = Light ray direction

// D = Microfacet NDF
// G = Shadowing and masking
// F = Fresnel

// Vis = G / (4*NoL*NoV)
// f = Microfacet specular BRDF = D*G*F / (4*NoL*NoV) = D*Vis*F


half3 Diffuse_Lambert( half3 DiffuseColor )
{
	return DiffuseColor * (1 / PI);
}

// [Burley 2012, "Physically-Based Shading at Disney"]
float3 Diffuse_Burley( float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoH )
{
	float FD90 = 0.5 + 2 * VoH * VoH * Roughness;
	float FdV = 1 + (FD90 - 1) * Pow5( 1 - NoV );
	float FdL = 1 + (FD90 - 1) * Pow5( 1 - NoL );
	return DiffuseColor * ( (1 / PI) * FdV * FdL );
}

// [Gotanda 2012, "Beyond a Simple Physically Based Blinn-Phong Model in Real-Time"]
float3 Diffuse_OrenNayar( float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoH )
{
	float a = Roughness * Roughness;
	float s = a;// / ( 1.29 + 0.5 * a );
	float s2 = s * s;
	float VoL = 2 * VoH * VoH - 1;		// double angle identity
	float Cosri = VoL - NoV * NoL;
	float C1 = 1 - 0.5 * s2 / (s2 + 0.33);
	float C2 = 0.45 * s2 / (s2 + 0.09) * Cosri * ( Cosri >= 0 ? rcp( max( NoL, NoV ) ) : 1 );
	return DiffuseColor / PI * ( C1 + C2 ) * ( 1 + Roughness * 0.5 );
}

// [Gotanda 2014, "Designing Reflectance Models for New Consoles"]
float3 Diffuse_Gotanda( float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoH )
{
	float a = Roughness * Roughness;
	float a2 = a * a;
	float F0 = 0.04;
	float VoL = 2 * VoH * VoH - 1;		// double angle identity
	float Cosri = VoL - NoV * NoL;
#if 1
	float a2_13 = a2 + 1.36053;
	float Fr = ( 1 - ( 0.542026*a2 + 0.303573*a ) / a2_13 ) * ( 1 - pow( 1 - NoV, 5 - 4*a2 ) / a2_13 ) * ( ( -0.733996*a2*a + 1.50912*a2 - 1.16402*a ) * pow( 1 - NoV, 1 + rcp(39*a2*a2+1) ) + 1 );
	//float Fr = ( 1 - 0.36 * a ) * ( 1 - pow( 1 - NoV, 5 - 4*a2 ) / a2_13 ) * ( -2.5 * Roughness * ( 1 - NoV ) + 1 );
	float Lm = ( max( 1 - 2*a, 0 ) * ( 1 - Pow5( 1 - NoL ) ) + min( 2*a, 1 ) ) * ( 1 - 0.5*a * (NoL - 1) ) * NoL;
	float Vd = ( a2 / ( (a2 + 0.09) * (1.31072 + 0.995584 * NoV) ) ) * ( 1 - pow( 1 - NoL, ( 1 - 0.3726732 * NoV * NoV ) / ( 0.188566 + 0.38841 * NoV ) ) );
	float Bp = Cosri < 0 ? 1.4 * NoV * NoL * Cosri : Cosri;
	float Lr = (21.0 / 20.0) * (1 - F0) * ( Fr * Lm + Vd + Bp );
	return DiffuseColor / PI * Lr;
#else
	float a2_13 = a2 + 1.36053;
	float Fr = ( 1 - ( 0.542026*a2 + 0.303573*a ) / a2_13 ) * ( 1 - pow( 1 - NoV, 5 - 4*a2 ) / a2_13 ) * ( ( -0.733996*a2*a + 1.50912*a2 - 1.16402*a ) * pow( 1 - NoV, 1 + rcp(39*a2*a2+1) ) + 1 );
	float Lm = ( max( 1 - 2*a, 0 ) * ( 1 - Pow5( 1 - NoL ) ) + min( 2*a, 1 ) ) * ( 1 - 0.5*a + 0.5*a * NoL );
	float Vd = ( a2 / ( (a2 + 0.09) * (1.31072 + 0.995584 * NoV) ) ) * ( 1 - pow( 1 - NoL, ( 1 - 0.3726732 * NoV * NoV ) / ( 0.188566 + 0.38841 * NoV ) ) );
	float Bp = Cosri < 0 ? 1.4 * NoV * Cosri : Cosri / max( NoL, 1e-8 );
	float Lr = (21.0 / 20.0) * (1 - F0) * ( Fr * Lm + Vd + Bp );
	return DiffuseColor / PI * Lr;
#endif
}

// [Portsmouth et al. 2025, "EON: A Practical Energy-Preserving Rough Diffuse BRDF"]
float3 Diffuse_EON( float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoL )
{
	// Albedo inversion for EON model to maintain a consistent color with lambert
	float3 Rho = DiffuseColor * (1.0 + (0.189468 - 0.189468 * DiffuseColor) * Roughness);

	// This is the main shaping term from the Oren-Nayar model (with tweaks by Fujii)
	float S = VoL - NoV * NoL;
	float SOverT = max(S * rcp(max(1e-6, max(NoV, NoL))), S);
	const float constant1_FON = 0.5f - 2.0f / (3.0f * PI);
	// AF = rcp(1 + Roughness * constant1_FON) is nearly a straight line, so approximate it as such
	float AF = 1 - Roughness * (1 - 1 / (1 + constant1_FON));
	float f_ss = AF * (1 + Roughness * SOverT);

	// 4th Order approximation from the paper is a bit too heavy, first order seems to work just as well
	const float g1 = 0.262048f;
	float GoverPi_V = g1 - g1 * NoV;
	// Use (1 - Eo) only as a non-reciprocal approach to energy conservation
	float f_ms = 1.0f - AF * (1 + Roughness * GoverPi_V);
	// The Rho_ms term from the paper can be approximated as just Rho^2
	return Rho * (f_ss + Rho * f_ms) * (1.0 / PI);
}

#define ROUGH_DIFFUSE_BRDF_VERSION 3
// This models a rough surface that has a GGX NDF where each microfacet has a lambertian response. Various models have been proposed
// to try and approximate this behavior.
float3 Diffuse_GGX_Rough( float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoH, float NoH, float RetroReflectivityWeight )
{
	// We saturate each input to avoid out of range negative values which would result in weird darkening at the edge of meshes (resulting from tangent space interpolation).
	NoV = saturate(NoV);
	NoL = saturate(NoL);
	VoH = saturate(VoH);
	NoH = saturate(NoH);
#if ROUGH_DIFFUSE_BRDF_VERSION == 3
	// It turns out the EON model in the range [0, 0.4] is nearly a perfect match to a ground truth
	// simulation of diffuse microfacets oriented with a GGX NDF.
	float VoL = 2 * VoH * VoH - 1;		// double angle identity to keep signature above consistent with other models
	return Diffuse_EON(DiffuseColor, RetroReflectivityWeight * Roughness * 0.4, NoV, NoL, VoL);
#elif ROUGH_DIFFUSE_BRDF_VERSION == 2
	// [ Chan 2024, "Multiscattering Diffuse and Specular BRDFs", Unpublished manuscript ]
	Roughness *= RetroReflectivityWeight;
	const float Alpha = Roughness * Roughness;
	// The original writeup uses an FSmooth term inspired by Burley diffuse to balance energy between spec/diffuse.
	// However in our implementation the energy balance between diffuse and spec is handled externally, so we stick
	// to a plain lambertian for the Roughness=0 limit.
	const float FSmooth = 1;
	const float Scale = max(0.55 - 0.2 * Roughness, 1.25 - 1.6 * Roughness);
	const float Bias = saturate(4 * Alpha);
	const float FRough = Scale * (NoH + Bias) * rcp(NoH + 0.025) * VoH * VoH;
	const float DiffuseSS = lerp(FSmooth, FRough, Roughness);
	const float DiffuseMS = Alpha * 0.38;
	return (1 / PI) * DiffuseColor * (DiffuseSS + DiffuseMS);
#else
	// [ Chan 2018, "Material Advances in Call of Duty: WWII" ]
	// It has been extended here to fade out retro reflectivity contribution from area light in order to avoid visual artefacts.
	float a2 = Pow4(Roughness);
	// a2 = 2 / ( 1 + exp2( 18 * g )
	float g = saturate( (1.0 / 18.0) * log2( 2 * rcpFast(a2) - 1 ) );

	float F0 = VoH + Pow5( 1 - VoH );
	float FdV = 1 - 0.75 * Pow5( 1 - NoV );
	float FdL = 1 - 0.75 * Pow5( 1 - NoL );

	// Rough (F0) to smooth (FdV * FdL) response interpolation
	float Fd = lerp( F0, FdV * FdL, saturate( 2.2 * g - 0.5 ) );

	// Retro reflectivity contribution.
	float Fb = ( (34.5 * g - 59 ) * g + 24.5 ) * VoH * exp2( -max( 73.2 * g - 21.2, 8.9 ) * sqrtFast( NoH ) );
	// It fades out when lights become area lights in order to avoid visual artefacts.
	Fb *= RetroReflectivityWeight;

	float Lobe = (1 / PI) * (Fd + Fb);

	// We clamp the BRDF lobe value to an arbitrary value of 1 to get some practical benefits at high roughness:
	// - This is to avoid too bright edges when using normal map on a mesh and the local bases, L, N and V ends up in an top emisphere setup.
	// - This maintains the full proper rough look of a sphere when not using normal maps.
	// - This also fixes the furnace test returning too much energy at the edge of a mesh.
	Lobe = min(1.0, Lobe);

	return DiffuseColor * Lobe;
#endif
}

// [Blinn 1977, "Models of light reflection for computer synthesized pictures"]
float D_Blinn( float a2, float NoH )
{
	float n = 2 / a2 - 2;
	return (n+2) / (2*PI) * PhongShadingPow( NoH, n );		// 1 mad, 1 exp, 1 mul, 1 log
}

// [Beckmann 1963, "The scattering of electromagnetic waves from rough surfaces"]
float D_Beckmann( float a2, float NoH )
{
	float NoH2 = NoH * NoH;
	return exp( (NoH2 - 1) / (a2 * NoH2) ) / ( PI * a2 * NoH2 * NoH2 );
}

// GGX / Trowbridge-Reitz
// [Walter et al. 2007, "Microfacet models for refraction through rough surfaces"]
float D_GGX( float a2, float NoH )
{
	float d = ( NoH * a2 - NoH ) * NoH + 1;	// 2 mad
	return a2 / ( PI*d*d );					// 4 mul, 1 rcp
}

// Anisotropic GGX
// [Burley 2012, "Physically-Based Shading at Disney"]
float D_GGXaniso( float ax, float ay, float NoH, float XoH, float YoH )
{
// The two formulations are mathematically equivalent
#if 1
	float a2 = ax * ay;
	float3 V = float3(ay * XoH, ax * YoH, a2 * NoH);
	float S = dot(V, V);

	return (1.0f / PI) * a2 * Square(a2 / S);
#else
	float d = XoH*XoH / (ax*ax) + YoH*YoH / (ay*ay) + NoH*NoH;
	return 1.0f / ( PI * ax*ay * d*d );
#endif
}

float Vis_Implicit()
{
	return 0.25;
}

// [Neumann et al. 1999, "Compact metallic reflectance models"]
float Vis_Neumann( float NoV, float NoL )
{
	return 1 / ( 4 * max( NoL, NoV ) );
}

// [Kelemen 2001, "A microfacet based coupled specular-matte brdf model with importance sampling"]
float Vis_Kelemen( float VoH )
{
	// constant to prevent NaN
	return rcp( 4 * VoH * VoH + 1e-5);
}

// Tuned to match behavior of Vis_Smith
// [Schlick 1994, "An Inexpensive BRDF Model for Physically-Based Rendering"]
float Vis_Schlick( float a2, float NoV, float NoL )
{
	float k = sqrt(a2) * 0.5;
	float Vis_SchlickV = NoV * (1 - k) + k;
	float Vis_SchlickL = NoL * (1 - k) + k;
	return 0.25 / ( Vis_SchlickV * Vis_SchlickL );
}

// Smith term for GGX
// [Smith 1967, "Geometrical shadowing of a random rough surface"]
float Vis_Smith( float a2, float NoV, float NoL )
{
	float Vis_SmithV = NoV + sqrt( NoV * (NoV - NoV * a2) + a2 );
	float Vis_SmithL = NoL + sqrt( NoL * (NoL - NoL * a2) + a2 );
	return rcp( Vis_SmithV * Vis_SmithL );
}

// Appoximation of joint Smith term for GGX
// [Heitz 2014, "Understanding the Masking-Shadowing Function in Microfacet-Based BRDFs"]
float Vis_SmithJointApprox( float a2, float NoV, float NoL )
{
	float a = sqrt(a2);
	float Vis_SmithV = NoL * ( NoV * ( 1 - a ) + a );
	float Vis_SmithL = NoV * ( NoL * ( 1 - a ) + a );
	return 0.5 * rcp( Vis_SmithV + Vis_SmithL );
}

// [Heitz 2014, "Understanding the Masking-Shadowing Function in Microfacet-Based BRDFs"]
float Vis_SmithJoint(float a2, float NoV, float NoL) 
{
	float Vis_SmithV = NoL * sqrt(NoV * (NoV - NoV * a2) + a2);
	float Vis_SmithL = NoV * sqrt(NoL * (NoL - NoL * a2) + a2);
	return 0.5 * rcp(Vis_SmithV + Vis_SmithL);
}

// [Heitz 2014, "Understanding the Masking-Shadowing Function in Microfacet-Based BRDFs"]
float Vis_SmithJointAniso(float ax, float ay, float NoV, float NoL, float XoV, float XoL, float YoV, float YoL)
{
	float Vis_SmithV = NoL * length(float3(ax * XoV, ay * YoV, NoV));
	float Vis_SmithL = NoV * length(float3(ax * XoL, ay * YoL, NoL));
	return 0.5 * rcp(Vis_SmithV + Vis_SmithL);
}

float3 F_None( float3 SpecularColor )
{
	return SpecularColor;
}

// [Schlick 1994, "An Inexpensive BRDF Model for Physically-Based Rendering"]
float3 F_Schlick( float3 SpecularColor, float VoH )
{
	float Fc = Pow5( 1 - VoH );					// 1 sub, 3 mul
	//return Fc + (1 - Fc) * SpecularColor;		// 1 add, 3 mad
	
	// Anything less than 2% is physically impossible and is instead considered to be shadowing
	return saturate( 50.0 * SpecularColor.g ) * Fc + (1 - Fc) * SpecularColor;
}

float3 F_Schlick(float3 F0, float3 F90, float VoH)
{
	float Fc = Pow5(1 - VoH);
	return F90 * Fc + (1 - Fc) * F0;
}

float3 F_AdobeF82(float3 F0, float3 F82, float VoH)
{
	// [Kutz et al. 2021, "Novel aspects of the Adobe Standard Material" ]
	// See Section 2.3 (note the formulas in the paper do not match the code, the code is the correct version)
	// The constants below are derived by just constant folding the terms dependent on CosThetaMax=1/7
	const float Fc = Pow5(1 - VoH);
	const float K = 49.0 / 46656.0;
	float3 b = (K - K * F82) * (7776.0 + 9031.0 * F0);
	return saturate(F0 + Fc * ((1 - F0) - b * (VoH - VoH * VoH)));
}

float3 F_Fresnel( float3 SpecularColor, float VoH )
{
	float3 SpecularColorSqrt = sqrt( clamp(SpecularColor, float3(0, 0, 0), float3(0.99, 0.99, 0.99) ) );
	float3 n = ( 1 + SpecularColorSqrt ) / ( 1 - SpecularColorSqrt );
	float3 g = sqrt( n*n + VoH*VoH - 1 );
	return 0.5 * Square( (g - VoH) / (g + VoH) ) * ( 1 + Square( ((g+VoH)*VoH - 1) / ((g-VoH)*VoH + 1) ) );
}

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedSheenLTCSampler	View.SharedBilinearClampedSampler
#else
	#define SharedSheenLTCSampler	InSheenSampler
#endif

float4 SheenLTC_Cofficients(float NoV, float Roughness, Texture2D InSheenLTCTexture, SamplerState InSheenSampler)
{
#if 1
	const float Alpha = sqrt(Roughness);
	const float SatNoV = saturate(abs(NoV) + 1e-5);
	float2 UV = float2(Alpha, SatNoV);
	UV = UV * (31.0 / 32.0) + (0.5 / 32.0);
	return InSheenLTCTexture.SampleLevel(SharedSheenLTCSampler, UV, 0);
#else
	// Sheen model LUT approximation with ALU under the MaterialX TPS.
	// SUBSTRATE_TODO: measure if this is worth it on some platforms.
	float x = saturate(abs(NoV) + 1e-5);
	float y = sqrt(Roughness);

	float A = ((2.58126 * x + 0.813703 * y) * y) / (1.0 + 0.310327 * x * x + 2.60994 * x * y);
	float B = sqrt(1.0 - x) * (y - 1.0) * y * y * y / (0.0000254053 + 1.71228 * x - 1.71506 * x * y + 1.34174 * y * y);

	float s = y * (0.0206607 + 1.58491 * y) / (0.0379424 + y * (1.32227 + y));
	float m = y * (-0.193854 + y * (-1.14885 + y * (1.7932 - 0.95943 * y * y))) / (0.046391 + y);
	float o = y * (0.000654023 + (-0.0207818 + 0.119681 * y) * y) / (1.26264 + y * (-1.92021 + y));
	float R = exp(-0.5 * Square((x - m) / s)) / (s * sqrt(2.0 * PI)) + o;

	return float4(A, B, R, 0.0f);
#endif
}

float SheenLTC_DirectionalAlbedo(float NoV, float Roughness, Texture2D InSheenLTCTexture, SamplerState InSheenSampler)
{
	return SheenLTC_Cofficients(NoV, Roughness, InSheenLTCTexture, InSheenSampler).z;
}

// [Zeltner et al. 2022, "Practical Multiple-Scattering Sheen Using Linearly Transformed Cosines"]
float3 SheenLTC_Sample(float3 CosSample, float3 V, half3 N, float NoV, float Roughness, Texture2D InSheenLTCTexture, SamplerState InSheenSampler)
{
	const float4 LTC = SheenLTC_Cofficients(NoV, Roughness, InSheenLTCTexture, InSheenSampler);
	const float aInv = LTC.x;
	const float bInv = LTC.y;
	// NOTE: The vector is scaled by aInv compared to the reference implementation to avoid unecessary divides
	const float3 LocalL = normalize(float3(CosSample.x - CosSample.z * bInv, CosSample.y, aInv * CosSample.z));

	// Rotate L into tangent space
	const float3 T1 = normalize(V - N * NoV);
	const float3 T2 = cross(N, T1);
	const float3x3 TangentBasis = float3x3(T1, T2, N);
	const float3 S = mul(LocalL, TangentBasis);
	return S;
}

// [Zeltner et al. 2022, "Practical Multiple-Scattering Sheen Using Linearly Transformed Cosines"]
// The "OutDirectionalAlbedo" should be multiplied by the result (it is kept separate because in the sampling case it needs to be only included in the weight, not the pdf)
float SheenLTC_Eval(float3 V, float3 L, half3 N, float NoV, float Roughness, Texture2D InSheenLTCTexture, SamplerState InSheenSampler, inout float OutDirectionalAlbedo)
{
	// Rotate L into tangent space
	const float3 T1 = normalize(V - N * NoV);
	const float3 T2 = cross(N, T1);
	const float3x3 TangentBasis = float3x3(T1, T2, N);

	// L' = mul(L, InvT) 
	// Since TangentBasis is orthnormal Inv(T)=Transpose(T), i.e., mul(L, transpose(T)), which is equivalent to mul(T,L)
	const float3 LocalL = mul(TangentBasis, L); 

	const float4 LTC = SheenLTC_Cofficients(NoV, Roughness, InSheenLTCTexture, InSheenSampler);
	const float aInv = LTC.x;
	const float bInv = LTC.y;
	OutDirectionalAlbedo = LTC.z;

	float3 WrappedLocalL = float3(
		aInv * LocalL.x + bInv * LocalL.z,
		aInv * LocalL.y,
		LocalL.z);

	const float InvLenWrappedLocalL2 = rcp(length2(WrappedLocalL));
	// Jabobian is defined as: det(InvLTC) / ||InvLTC . L||
	// Jacobian should be aInv^2/Length^3 but since we skip the normalization
	// we end up with aInv^2/Length^4 which is cheaper to compute
	const float Jacobian = Pow2(aInv * InvLenWrappedLocalL2);


	// Evaluation a normalized cos distribution (note: WrappedLocalL was not normalized, we folded the extra length division into the Jacobian)
	const float CosDistribution = WrappedLocalL.z / PI;
	const float Out = CosDistribution * Jacobian;

	return max(Out, 0.f);
}

//---------------
// EnvBRDF
//---------------
void ModifyGGXAnisotropicNormalRoughness(float3 WorldTangent, float Anisotropy, inout float Roughness, inout float3 N, float3 V)
{
	if (abs(Anisotropy) > 0.0f)
	{
		float3 X = WorldTangent;
		float3 Y = normalize(cross(N, X));

		float3 AnisotropicDir = Anisotropy >= 0.0f ? Y : X;
		float3 AnisotropicT   = cross(AnisotropicDir, V); 
		float3 AnisotropicN   = cross(AnisotropicT, AnisotropicDir);

		float AnisotropicStretch = abs(Anisotropy) * saturate(5.0f * Roughness);
		N = normalize(lerp(N, AnisotropicN, AnisotropicStretch));
#if 0
		Roughness *= saturate(1.2f - abs(Anisotropy));
#endif
	}
}

// Convert a roughness and an anisotropy factor into GGX alpha values respectively for the major and minor axis of the tangent frame
void GetAnisotropicRoughness(float Alpha, float Anisotropy, out float ax, out float ay)
{
#if 1
	// Anisotropic parameters: ax and ay are the roughness along the tangent and bitangent	
	// Kulla 2017, "Revisiting Physically Based Shading at Imageworks"
	ax = max(Alpha * (1.0 + Anisotropy), 0.001f);
	ay = max(Alpha * (1.0 - Anisotropy), 0.001f);
#else
	float K = sqrt(1.0f - 0.95f * Anisotropy);
	ax = max(Alpha / K, 0.001f);
	ay = max(Alpha * K, 0.001f);
#endif
}

// Convert a roughness and an anisotropy factor, into two roughnesses respectively for the major and minor axis of the tangent frame
float2 GetAnisotropicRoughness(float Roughness, float Anisotropy)
{
	// Anisotropic parameters: ax and ay are the roughness along the tangent and bitangent	
	// Kulla 2017, "Revisiting Physically Based Shading at Imageworks"
	float2 Out = saturate(Roughness);
	Anisotropy = clamp(Anisotropy, -1.0, 1.0);
	Out.x = max(Roughness * sqrt(1.0 + Anisotropy), 0.001f);
	Out.y = max(Roughness * sqrt(1.0 - Anisotropy), 0.001f);
	return Out;
}

// Return anisotropy factor (+1: Tangent is major axis, -1: Bitangent is the major axis), and the original roughness
void GetAnisotropicFactor(float RoughnessX, float RoughnessY, inout float Anisotropy, inout float OriginalRoughness)
{
	const float MinRoughness = 0.001f;
	float r = Pow2(max(RoughnessX, MinRoughness) / max(RoughnessY, MinRoughness));
	Anisotropy = (r - 1.0) / (r + 1.0);
	OriginalRoughness = (RoughnessX + RoughnessY) / (sqrt(1.0 + Anisotropy) + sqrt(1.0 - Anisotropy));
}

#ifndef PreIntegratedGF
Texture2D		PreIntegratedGF;
SamplerState	PreIntegratedGFSampler;
#endif

// [Karis 2013, "Real Shading in Unreal Engine 4" slide 11]
half3 EnvBRDF( half3 SpecularColor, half Roughness, half NoV )
{
	// Importance sampled preintegrated G * F
	float2 AB = Texture2DSampleLevel( PreIntegratedGF, PreIntegratedGFSampler, float2( NoV, Roughness ), 0 ).rg;

	// Anything less than 2% is physically impossible and is instead considered to be shadowing 
	float3 GF = SpecularColor * AB.x + saturate( 50.0 * SpecularColor.g ) * AB.y;
	return GF;
}

half3 EnvBRDF(half3 F0, half3 F90, half Roughness, half NoV)
{
	// Importance sampled preintegrated G * F
	float2 AB = Texture2DSampleLevel(PreIntegratedGF, PreIntegratedGFSampler, float2(NoV, Roughness), 0).rg;
	float3 GF = F0 * AB.x + F90 * AB.y;
	return GF;
}

half2 EnvBRDFApproxLazarov(half Roughness, half NoV)
{
	// [ Lazarov 2013, "Getting More Physical in Call of Duty: Black Ops II" ]
	// Adaptation to fit our G term.
	const half4 c0 = { -1, -0.0275, -0.572, 0.022 };
	const half4 c1 = { 1, 0.0425, 1.04, -0.04 };
	half4 r = Roughness * c0 + c1;
	half a004 = min(r.x * r.x, exp2(-9.28 * NoV)) * r.x + r.y;
	half2 AB = half2(-1.04, 1.04) * a004 + r.zw;
	return AB;
}

half3 EnvBRDFApprox( half3 SpecularColor, half Roughness, half NoV )
{
	half2 AB = EnvBRDFApproxLazarov(Roughness, NoV);

	// Anything less than 2% is physically impossible and is instead considered to be shadowing
	// Note: this is needed for the 'specular' show flag to work, since it uses a SpecularColor of 0
	float F90 = saturate( 50.0 * SpecularColor.g );

	return SpecularColor * AB.x + F90 * AB.y;
}

half3 EnvBRDFApprox(half3 F0, half3 F90, half Roughness, half NoV)
{
	half2 AB = EnvBRDFApproxLazarov(Roughness, NoV);
	return F0 * AB.x + F90 * AB.y;
}

half EnvBRDFApproxNonmetal( half Roughness, half NoV )
{
	// Same as EnvBRDFApprox( 0.04, Roughness, NoV )
	const half2 c0 = { -1, -0.0275 };
	const half2 c1 = { 1, 0.0425 };
	half2 r = Roughness * c0 + c1;
	return min( r.x * r.x, exp2( -9.28 * NoV ) ) * r.x + r.y;
}

void EnvBRDFApproxFullyRough(inout half3 DiffuseColor, inout half3 SpecularColor)
{
	// Factors derived from EnvBRDFApprox( SpecularColor, 1, 1 ) == SpecularColor * 0.4524 - 0.0024
	DiffuseColor += SpecularColor * 0.45;
	SpecularColor = 0;
	// We do not modify Roughness here as this is done differently at different places.
}
void EnvBRDFApproxFullyRough(inout half3 DiffuseColor, inout half SpecularColor)
{
	DiffuseColor += SpecularColor * 0.45;
	SpecularColor = 0;
}
void EnvBRDFApproxFullyRough(inout half3 DiffuseColor, inout half3 F0, inout half3 F90)
{
	DiffuseColor += F0 * 0.45;
	F0 = F90 = 0;
}


float D_InvBlinn( float a2, float NoH )
{
	float A = 4;
	float Cos2h = NoH * NoH;
	float Sin2h = 1 - Cos2h;
	//return rcp( PI * (1 + A*m2) ) * ( 1 + A * ClampedPow( Sin2h, 1 / m2 - 1 ) );
	return rcp( PI * (1 + A*a2) ) * ( 1 + A * exp( -Cos2h / a2 ) );
}

float D_InvBeckmann( float a2, float NoH )
{
	float A = 4;
	float Cos2h = NoH * NoH;
	float Sin2h = 1 - Cos2h;
	float Sin4h = Sin2h * Sin2h;
	return rcp( PI * (1 + A*a2) * Sin4h ) * ( Sin4h + A * exp( -Cos2h / (a2 * Sin2h) ) );
}

float D_InvGGX( float a2, float NoH )
{
	float A = 4;
	float d = ( NoH - a2 * NoH ) * NoH + a2;
	return rcp( PI * (1 + A*a2) ) * ( 1 + 4 * a2*a2 / ( d*d ) );
}

float Vis_Cloth( float NoV, float NoL )
{
	return rcp( 4 * ( NoL + NoV - NoL * NoV ) );
}

float D_Charlie(float Roughness, float NoH)
{
	float InvR = 1 / Roughness;
	float Cos2H = NoH * NoH;
	float Sin2H = 1 - Cos2H;
	return (2 + InvR) * pow(Sin2H, InvR * 0.5) / (2 * PI);
}

// [Estevez and Kulla 2017, "Production Friendly Microfacet Sheen BRDF"]
float Vis_Charlie_L(float x, float r)
{
	r = saturate(r);
	r = 1.0 - (1. - r) * (1. - r);

	float a = lerp(25.3245 , 21.5473 , r);
	float b = lerp( 3.32435,  3.82987, r);
	float c = lerp( 0.16801,  0.19823, r);
	float d = lerp(-1.27393, -1.97760, r);
	float e = lerp(-4.85967, -4.32054, r);

	return a * rcp( (1 + b * pow(x, c)) + d * x + e);
}
float Vis_Charlie(float Roughness, float NoV, float NoL)
{
	float VisV = NoV < 0.5 ? exp(Vis_Charlie_L(NoV, Roughness)) : exp(2 * Vis_Charlie_L(0.5, Roughness) - Vis_Charlie_L(1 - NoV, Roughness));
	float VisL = NoL < 0.5 ? exp(Vis_Charlie_L(NoL, Roughness)) : exp(2 * Vis_Charlie_L(0.5, Roughness) - Vis_Charlie_L(1 - NoL, Roughness));

	return rcp(((1 + VisV + VisL) * (4 * NoV * NoL)));
}

float Vis_Ashikhmin(float NoV, float NoL)
{
	return rcp(4 * (NoL + NoV - NoL * NoV));
}

// Special medium transmittance evaluation for the clear coat BSDF.
float3 SimpleClearCoatTransmittance(float NoL, float NoV, float Metallic, float3 BaseColor)
{
	float3 Transmittance = 1.0;

	float ClearCoatCoverage = Metallic;
	if (ClearCoatCoverage > 0.0)
	{
		float LayerThickness = 1.0;	// Assume a normalized thickness

		// The path length inside the medium is from light to bottom surface, then back from bottom surface to the viewer.
		// This assuming an infinite and flat medium slab.
		float ThinDistance = LayerThickness * (rcp(NoV) + rcp(NoL));

		// BaseColor represents reflected color viewed at 0 incidence angle for the view and light direction... 
		float3 TransmittanceColor = Diffuse_Lambert(BaseColor);
		// ... and after being affected by the substrate transmittance when traveling through it, back and forth.
		// Because of this, extinction is normalized by traveling through layer thickness twice
		float3 ExtinctionCoefficient = -log(max(TransmittanceColor,0.0001)) / (2.0 * LayerThickness);

		// Optical depth is evaluated for ThinDistance computed above. But we also subtract 2.0 * LayerThickness
		// in order to respect the fact that BaseColor represents reflected color viewed at 0 incidence angle. => In this case, OpticalDepth will be 0 and thus Transmittance=1.
		float3 OpticalDepth = ExtinctionCoefficient * max(ThinDistance - 2.0 * LayerThickness, 0.0);

		// Compute transmittance as a fonction of OpticalDepth
		Transmittance = exp(-OpticalDepth);
		// And as a fucntion of the ClearCoatCoverage.
		Transmittance = lerp(1.0, Transmittance, ClearCoatCoverage);
	}

	// Note: this is a really special way to handle clear coat transmittance.
	//  - The basecolor of the bottom layer is used as the transmittance color.
	//	- So basically, a colored metal as bottom layer will look even more saturated at grazing angle.
	//	- Transmittance is only computed for metals, for some reason plastic do not receive transmittance.

	return Transmittance;
}

// Conversion of GGX roughness to Beckmann roughness
// Using Microfacet Distribution Function: To Change or Not to Change, That Is the Question.
float ConvertRoughnessGGXToBeckmann(float Roughness)
{
	// Use a custom remapping to better match the overall highlight shape, in particular for lower roughness (i.e., <0.5). 
	// The original mapping was Ctr=Roughness based on the paper
	const float Ctr = pow(Roughness, 0.7f);
	return saturate(0.8388 * Ctr+ 0.7 * Pow4(Ctr));
}
float2 ConvertRoughnessGGXToBeckmann(float2 Roughness)
{
	return float2(ConvertRoughnessGGXToBeckmann(Roughness.x), ConvertRoughnessGGXToBeckmann(Roughness.y));
}

struct FBeckmannDesc
{
	float2 Roughness;	// Roughness
	float2 Alpha;		// Roughness squared
	float2 Sigma;		// RMS slope of the microfacets
	float  Rho;			// Slope correlation factor to be used with LEAN mapping. See http://igg.unistra.fr/People/chermain/assets/pdf/Chermain2021RealTime.pdf.
};
FBeckmannDesc GGXToBeckmann(float2 GGXSafeRoughness)
{
	const float InvSqrt2 = 0.707106f;

	FBeckmannDesc Out;
	Out.Roughness	= ConvertRoughnessGGXToBeckmann(GGXSafeRoughness);
	Out.Alpha		= Square(Out.Roughness);
	Out.Sigma		= Out.Alpha * InvSqrt2;
	Out.Rho			= 0.0f;
	return Out;
}

// Hanrahan-Krueger BRDF (not including the phase function and albedo):
//   [Siggraph 1993, "Reflection from Layered Surfaces due to Subsurface Scattering", Pat Hanrahan, Wolfgang Krueger ]
//   http://www.graphics.stanford.edu/papers/subsurface/
// The same BRDF is used in the following paper which we reference for the more modern notation and for having numbered equations:
//   [EGSR 2014, "A Physically-Based BSDF for Modeling the Appearance of Paper", Marios Papas, Krystle de Mesa, and Henrik Wann Jensen ]
//   http://graphics.ucsd.edu/~henrik/papers/paper_appearance.pdf
float3 HanrahanKrueger_Weight(float NoV, float NoL, float3 Tr)
{
	float3 SigmaT = -log(max(Tr, 1e-5));
	if (NoV > 0)
	{
		if (NoL > 0)
		{
			// Section 4.4, Eq 8
			return (1 - exp(-SigmaT * (rcp(NoL) + rcp(NoV)))) * NoL / (NoV + NoL);
		}
		else if (NoL < 0)
		{
			if (abs(NoV + NoL) > 1e-5)
			{
				// Section 4.5, Eq 11
				return (exp(-SigmaT / NoV) - exp(SigmaT / NoL)) * -NoL / (NoV + NoL);
			}
			else
			{
				// Section 4.5, Eq 12
				return SigmaT * exp(-SigmaT / NoV) / NoV;
			}
		}
	}
	return 0;
}