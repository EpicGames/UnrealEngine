// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DualNumber.ush"

#ifndef NANITE_TESSELLATION
#define NANITE_TESSELLATION 0
#endif

typedef TDual< float3 > FBarycentrics;

template< typename T >
TDual<T> Lerp( T Value0, T Value1, T Value2, FBarycentrics Barycentrics )
{
	TDual<T> Result;
	Result.Value	= Value0 * Barycentrics.Value.x    + Value1 * Barycentrics.Value.y    + Value2 * Barycentrics.Value.z;
	Result.Value_dx	= Value0 * Barycentrics.Value_dx.x + Value1 * Barycentrics.Value_dx.y + Value2 * Barycentrics.Value_dx.z;
	Result.Value_dy	= Value0 * Barycentrics.Value_dy.x + Value1 * Barycentrics.Value_dy.y + Value2 * Barycentrics.Value_dy.z;

#if NANITE_TESSELLATION
	Result.Value = INVARIANT( Result.Value );
#endif

	return Result;
}

float3 GetSegmentBarycentrics(float2 PixelClip, float4 PointClip0, float4 PointClip1)
{
	const float2 RcpW = rcp(float2(PointClip0.w, PointClip1.w));
	const float3 Pos0 = PointClip0.xyz * RcpW.x;
	const float3 Pos1 = PointClip1.xyz * RcpW.y;
	const float2 P10  = Pos1.xy - Pos0.xy;
	const float InvNorm= rcp(dot(P10, P10));
	const float2 PRel = PixelClip - Pos0.xy;
	const float S = clamp(dot(P10, PRel) * InvNorm, 0.f, 1.f);

	const float u = 1 - S;
	const float v = S;
	const float u_perspective = u * RcpW.x;
	const float v_perspective = v * RcpW.y;
	const float denom = rcp(u_perspective + v_perspective);
	return float3(u_perspective * denom, v_perspective * denom, 0);
}

/** Calculates perspective correct barycentric coordinates and partial derivatives for a segment with a U Coordinate. */
FBarycentrics CalculateSegmentBarycentrics(float2 PixelClip, float4 PointClip0, float4 PointClip1, float2 ViewInvSize)
{
	const float3 V   = GetSegmentBarycentrics(PixelClip,                                   PointClip0, PointClip1);
	const float3 V10 = GetSegmentBarycentrics(PixelClip + float2(2.0f * ViewInvSize.x, 0), PointClip0, PointClip1);
	const float3 V01 = GetSegmentBarycentrics(PixelClip + float2(0,-2.0f * ViewInvSize.y), PointClip0, PointClip1);

	FBarycentrics Out;
	Out.Value    = V;
	Out.Value_dx = V10 - V;
	Out.Value_dy = V01 - V;
	return Out;
}

/** Calculates perspective correct barycentric coordinates and partial derivatives using screen derivatives. */
FBarycentrics CalculateTriangleBarycentrics(float2 PixelClip, float4 PointClip0, float4 PointClip1, float4 PointClip2, float2 ViewInvSize)
{
	FBarycentrics Barycentrics;

	const float3 RcpW = rcp(float3(PointClip0.w, PointClip1.w, PointClip2.w));
	const float3 Pos0 = PointClip0.xyz * RcpW.x;
	const float3 Pos1 = PointClip1.xyz * RcpW.y;
	const float3 Pos2 = PointClip2.xyz * RcpW.z;

	const float3 Pos120X = float3(Pos1.x, Pos2.x, Pos0.x);
	const float3 Pos120Y = float3(Pos1.y, Pos2.y, Pos0.y);
	const float3 Pos201X = float3(Pos2.x, Pos0.x, Pos1.x);
	const float3 Pos201Y = float3(Pos2.y, Pos0.y, Pos1.y);

	const float3 C_dx = Pos201Y - Pos120Y;
	const float3 C_dy = Pos120X - Pos201X;

	const float3 C = C_dx * (PixelClip.x - Pos120X) + C_dy * (PixelClip.y - Pos120Y);	// Evaluate the 3 edge functions
	const float3 G = C * RcpW;

	const float H = dot(C, RcpW);
	const float RcpH = rcp(H);

	// UVW = C * RcpW / dot(C, RcpW)
	Barycentrics.Value = G * RcpH;

	// Texture coordinate derivatives:
	// UVW = G / H where G = C * RcpW and H = dot(C, RcpW)
	// UVW' = (G' * H - G * H') / H^2
	// float2 TexCoordDX = UVW_dx.y * TexCoord10 + UVW_dx.z * TexCoord20;
	// float2 TexCoordDY = UVW_dy.y * TexCoord10 + UVW_dy.z * TexCoord20;
	const float3 G_dx = C_dx * RcpW;
	const float3 G_dy = C_dy * RcpW;

	const float H_dx = dot(C_dx, RcpW);
	const float H_dy = dot(C_dy, RcpW);

	Barycentrics.Value_dx = (G_dx * H - G * H_dx) * (RcpH * RcpH) * ( 2.0f * ViewInvSize.x);
	Barycentrics.Value_dy = (G_dy * H - G * H_dy) * (RcpH * RcpH) * (-2.0f * ViewInvSize.y);

	return Barycentrics;
}

float3 RayTriangleIntersectBarycentrics( float3 o, float3 rd, float3 v0, float3 v1, float3 v2 )
{
	// Muller-Trumbore ray triangle intersect
	float3 Edge01 = v1 - v0;
	float3 Edge02 = v2 - v0;
	float3 Origin0 = o - v0;

	float3 Dirx02 = cross( rd, Edge02 );
	float InvDet = 1.0 / dot( Edge01, Dirx02 );

	float3 UVW;
	UVW.y = InvDet * dot( Origin0, Dirx02 );
	UVW.z = InvDet * dot( rd, cross( Origin0, Edge01) );
	UVW.x = 1.0 - UVW.y - UVW.z;
	return UVW;
}

float3 ClosestPointBarycentrics( float3 p, float3 v0, float3 v1, float3 v2 )
{
	// Project on to triangle along normal
	float3 TriNormal = cross( v2 - v0, v1 - v0 );
	
	return RayTriangleIntersectBarycentrics(p, TriNormal, v0, v1, v2);
}

FBarycentrics CalculateTriangleBarycentrics(
	float3 CameraLocal,
	float3 PositionPixel,
	float3 PositionPixelX,
	float3 PositionPixelY,
	float3 Position0,
	float3 Position1,
	float3 Position2,
	float3 Normal0,
	float3 Normal1,
	float3 Normal2 )
{
	/*
		Displacement mapping

		PositionPixel =
			U * ( Position0 + Normal0 * Height ) +
			V * ( Position1 + Normal1 * Height ) +
			W * ( Position2 + Normal2 * Height );

		Solve for Height
	*/

	// 4 looks perfect but 2 seems good enough
	uint NumIterations = 2;
	for( uint j = 0; j < NumIterations; j++ )
	{
		float3 UVW = ClosestPointBarycentrics( PositionPixel, Position0, Position1, Position2 );
		UVW = max( 0, UVW );
		UVW /= dot( UVW, 1 );

		float3 ClosestPoint;
		ClosestPoint  = Position0 * UVW.x;
		ClosestPoint += Position1 * UVW.y;
		ClosestPoint += Position2 * UVW.z;

		float3 ClosestNormal;
		ClosestNormal  = Normal0 * UVW.x;
		ClosestNormal += Normal1 * UVW.y;
		ClosestNormal += Normal2 * UVW.z;

		float Displace = dot( PositionPixel - ClosestPoint, ClosestNormal ) / dot( ClosestNormal, ClosestNormal );

		Position0 += Normal0 * Displace;
		Position1 += Normal1 * Displace;
		Position2 += Normal2 * Displace;
	}

	FBarycentrics Barycentrics;
	Barycentrics.Value		= ClosestPointBarycentrics( PositionPixel,  Position0, Position1, Position2 );

	// Take estimate for position at dx and dy and project onto the displaced triangle along the view ray
	// We do this to correct for not including any the depth delta in our original estimation
	Barycentrics.Value_dx	= RayTriangleIntersectBarycentrics( CameraLocal, PositionPixelX - CameraLocal, Position0, Position1, Position2 ) - Barycentrics.Value;
	Barycentrics.Value_dy	= RayTriangleIntersectBarycentrics( CameraLocal, PositionPixelY - CameraLocal, Position0, Position1, Position2 ) - Barycentrics.Value;

	return Barycentrics;
}