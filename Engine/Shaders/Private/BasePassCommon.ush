// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassCommon.ush: Base pass definitions used by both vertex and pixel shader
=============================================================================*/

#if SHADING_PATH_DEFERRED
#if MATERIALBLENDING_ANY_TRANSLUCENT
	#define ForwardLightStruct TranslucentBasePass.Shared.Forward
	#define ForwardDirLightShadowStruct TranslucentBasePass.Shared.ForwardDirLightShadow
	#define ReflectionStruct TranslucentBasePass.Shared.Reflection
	#define PlanarReflectionStruct TranslucentBasePass.Shared.PlanarReflection
	#define FogStruct TranslucentBasePass.Shared.Fog
	#define LFVStruct TranslucentBasePass.Shared
#else
	#define ForwardLightStruct OpaqueBasePass.Shared.Forward
	#define ForwardDirLightShadowStruct OpaqueBasePass.Shared.ForwardDirLightShadow
	#define ReflectionStruct OpaqueBasePass.Shared.Reflection
	#define PlanarReflectionStruct OpaqueBasePass.Shared.PlanarReflection
	#define FogStruct OpaqueBasePass.Shared.Fog
	#define LFVStruct OpaqueBasePass.Shared
#endif 

#undef NEEDS_LIGHTMAP_COORDINATE

#define NEEDS_LIGHTMAP_COORDINATE		(HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP)
// Translucent materials need to compute fogging in the forward shading pass
// Materials that read from scene color skip getting fogged, because the contents of the scene color lookup have already been fogged
// This is not foolproof, as any additional color the material adds will then not be fogged correctly
#define TRANSLUCENCY_NEEDS_BASEPASS_FOGGING	(MATERIAL_ENABLE_TRANSLUCENCY_FOGGING && MATERIALBLENDING_ANY_TRANSLUCENT && !MATERIAL_USES_SCENE_COLOR_COPY)
// With forward shading, fog always needs to be computed in the base pass to work correctly with MSAA
#define OPAQUE_NEEDS_BASEPASS_FOGGING		(!MATERIALBLENDING_ANY_TRANSLUCENT && FORWARD_SHADING)

#define NEEDS_BASEPASS_VERTEX_FOGGING		(TRANSLUCENCY_NEEDS_BASEPASS_FOGGING && !MATERIAL_COMPUTE_FOG_PER_PIXEL || OPAQUE_NEEDS_BASEPASS_FOGGING && PROJECT_VERTEX_FOGGING_FOR_OPAQUE)
#define NEEDS_BASEPASS_PIXEL_FOGGING		(TRANSLUCENCY_NEEDS_BASEPASS_FOGGING && MATERIAL_COMPUTE_FOG_PER_PIXEL || OPAQUE_NEEDS_BASEPASS_FOGGING && !PROJECT_VERTEX_FOGGING_FOR_OPAQUE)

// Volumetric fog interpolated per vertex gives very poor results, always sample the volumetric fog texture per-pixel
// Opaque materials in the deferred renderer get volumetric fog applied in a deferred fog pass
#define NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING		(TRANSLUCENCY_NEEDS_BASEPASS_FOGGING || FORWARD_SHADING)

#define NEEDS_LIGHTMAP						(NEEDS_LIGHTMAP_COORDINATE)

#define USES_GBUFFER						(FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !FORWARD_SHADING)

// Only some shader models actually need custom data.
#define WRITES_CUSTOMDATA_TO_GBUFFER		(USES_GBUFFER && (MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN || MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_CLEAR_COAT || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || MATERIAL_SHADINGMODEL_HAIR || MATERIAL_SHADINGMODEL_CLOTH || MATERIAL_SHADINGMODEL_EYE || MATERIAL_SHADINGMODEL_TOON))

// Based on GetPrecomputedShadowMasks()
// Note: WRITES_PRECSHADOWFACTOR_TO_GBUFFER is currently disabled because we use the precomputed shadow factor GBuffer outside of STATICLIGHTING_TEXTUREMASK to store UseSingleSampleShadowFromStationaryLights
#define GBUFFER_HAS_PRECSHADOWFACTOR		(USES_GBUFFER && ALLOW_STATIC_LIGHTING)
#define WRITES_PRECSHADOWFACTOR_ZERO		(!(STATICLIGHTING_TEXTUREMASK && STATICLIGHTING_SIGNEDDISTANCEFIELD) && (HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP))
#define WRITES_PRECSHADOWFACTOR_TO_GBUFFER	(GBUFFER_HAS_PRECSHADOWFACTOR && !WRITES_PRECSHADOWFACTOR_ZERO)

// If a primitive has static lighting, we assume it is not moving. If it is, it will be rerendered in an extra renderpass.
#define SUPPORTS_WRITING_VELOCITY_TO_BASE_PASS	(FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED))
#define WRITES_VELOCITY_TO_GBUFFER				((SUPPORTS_WRITING_VELOCITY_TO_BASE_PASS || USES_GBUFFER) && GBUFFER_HAS_VELOCITY && (!COMPUTE_SHADED || USES_BASE_PASS_VELOCITY || USES_WORLD_POSITION_OFFSET || USES_FIRST_PERSON_INTERPOLATION))

#define TRANSLUCENCY_ANY_PERVERTEX_LIGHTING (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL)
#define TRANSLUCENCY_ANY_VOLUMETRIC (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_ANY_PERVERTEX_LIGHTING)
#define TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME (!FORWARD_SHADING && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL))
#define TRANSLUCENCY_PERVERTEX_FORWARD_SHADING (FORWARD_SHADING && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL))

// We consider that the cloud shadow is always enabled but the interpolator is only used when render water or lit translucent with forward shading. We only support per vertex cloud shadow as of today.
#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
// Checking for SINGLE_LAYER_WATER_SHADING_QUALITY disables cloud shadows on low end platforms.
#define NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR (SUPPORT_CLOUD_SHADOW_ON_SINGLE_LAYER_WATER && (SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_FULL))
#else
#define NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR (SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT && IS_MATERIAL_TRANSLUCENT_AND_LIT && !TRANSLUCENCY_PERVERTEX_FORWARD_SHADING && !TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME)
#endif

// Local fog volumes are always evaluated per vertex for now, even with forward shading. But never for opaque meshes rendered in forward.
#define LOCAL_FOG_VOLUME_ON_TRANSLUCENT (PROJECT_LOCALFOGVOLUME_APPLYONTRANSLUCENT && (NEEDS_BASEPASS_VERTEX_FOGGING && !(OPAQUE_NEEDS_BASEPASS_FOGGING && PROJECT_VERTEX_FOGGING_FOR_OPAQUE)) && !MATERIAL_IS_SKY)

struct FSharedBasePassInterpolants
{
	// This is enabled for vertex fogging.
	// For texture-lightmapped translucency we can pass the vertex fog in its own interpolator
#if NEEDS_BASEPASS_VERTEX_FOGGING
	float4 VertexFog		: TEXCOORD7;
#endif

#if NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
	float  VertexCloudShadow: TEXCOORD8;
#endif

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	float3 PixelPositionExcludingWPO : TEXCOORD9;
#endif

#if TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME
	float3 AmbientLightingVector : TEXCOORD12;
#endif

#if TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME && TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
	float3 DirectionalLightingVector : TEXCOORD13;
#endif

#if TRANSLUCENCY_PERVERTEX_FORWARD_SHADING
	float3 VertexDiffuseLighting : TEXCOORD12;
#endif

#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
	#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL
		float3 VertexIndirectAmbient : TEXCOORD14;
	#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
		float4 VertexIndirectSH[3] : TEXCOORD14;
	#endif
#endif

#if WRITES_VELOCITY_TO_GBUFFER && !IS_NANITE_PASS
	// .xy is clip position, pre divide by w; .w is clip W; .z is 0 or 1 to mask out the velocity output
	INVARIANT_OUTPUT float4 VelocityPrevScreenPosition : VELOCITY_PREV_POS;
#endif
};

#define FBasePassInterpolantsVSToPS FSharedBasePassInterpolants

#endif

#define TranslucencyLightingVolumeAmbientInner TranslucentBasePass.TranslucencyLightingVolumeAmbientInner
#define TranslucencyLightingVolumeAmbientOuter TranslucentBasePass.TranslucencyLightingVolumeAmbientOuter
#define TranslucencyLightingVolumeDirectionalInner TranslucentBasePass.TranslucencyLightingVolumeDirectionalInner
#define TranslucencyLightingVolumeDirectionalOuter TranslucentBasePass.TranslucencyLightingVolumeDirectionalOuter
#define TranslucencySpecularDirectionInner TranslucentBasePass.TranslucencySpecularDirectionInner
#define TranslucencySpecularDirectionOuter TranslucentBasePass.TranslucencySpecularDirectionOuter

#define TranslucencyLightingRandomPositionOffsetRadius TranslucentBasePass.TranslucencyLightingRandomPositionOffsetRadius
#define TranslucencyLightingForwardUsingVolume TranslucentBasePass.TranslucencyLightingForwardUsingVolume

#include "TranslucencyVolumeCommon.ush"

#include "SphericalGaussian.ush"

struct FShadingOcclusion
{
	half  DiffOcclusion;
	half  SpecOcclusion;
	half3 BentNormal;
};

float DotSpecularSG( float Roughness, float3 N, float3 V, FSphericalGaussian LightSG )
{
	float a = Pow2( max( 0.02, Roughness ) );
	float a2 = a*a;
	
	float3 L = LightSG.Axis;
	float3 H = normalize(V + L);

	float NoV = saturate( abs( dot(N, V) ) + 1e-5 );

	FSphericalGaussian NDF;
	NDF.Axis = N;
	NDF.Sharpness = 2 / a2;
	NDF.Amplitude = rcp( PI * a2 );

#if 0
	{
		// Reflect NDF
		//float3 R = 2 * dot( V, N ) * N - V;
		float3 R = 2 * NoV * N - V;

		// Point lobe in off-specular peak direction
		//R = lerp( N, R, (1 - a) * ( sqrt(1 - a) + a ) );
		//R = normalize( R );

	#if 0
		// Warp
		FSphericalGaussian SpecularSG;
		SpecularSG.Axis = R;
		SpecularSG.Sharpness = 0.5 / ( a2 * max( NoV, 0.1 ) );
		SpecularSG.Amplitude = rcp( PI * a2 );
	#else
		FAnisoSphericalGaussian SpecularSG;
		SpecularSG.AxisZ = R;
		SpecularSG.AxisX = normalize( cross( N, SpecularSG.AxisZ ) );
		SpecularSG.AxisY = normalize( cross( R, SpecularSG.AxisX ) );
 
		// Second derivative of the sharpness with respect to how
		// far we are from basis Axis direction
		SpecularSG.SharpnessX = 0.25 / ( a2 * Pow2( max( NoV, 0.001 ) ) );
		SpecularSG.SharpnessY = 0.25 / a2;
		SpecularSG.Amplitude = rcp( PI * a2 );
	#endif
		return Dot( SpecularSG, LightSG );
	}
#elif 0
	{
		// Project LightSG into half vector space
	#if 0
		FSphericalGaussian WarpedLightSG;
		WarpedLightSG.Axis		= H;
		WarpedLightSG.Sharpness	= LightSG.Sharpness * 1.5 * NoV;
		WarpedLightSG.Amplitude	= LightSG.Amplitude;
	#else
		FAnisoSphericalGaussian WarpedLightSG;
		WarpedLightSG.AxisZ = H;
		WarpedLightSG.AxisX = normalize( cross( N, WarpedLightSG.AxisZ ) );
		WarpedLightSG.AxisY = normalize( cross( H, WarpedLightSG.AxisX ) );
 
		// Second derivative of the sharpness with respect to how
		// far we are from basis Axis direction
		WarpedLightSG.SharpnessX = LightSG.Sharpness * 2 * Pow2( NoV );
		WarpedLightSG.SharpnessY = LightSG.Sharpness * 2;
		WarpedLightSG.Amplitude	= LightSG.Amplitude;
	#endif

		return Dot( WarpedLightSG, NDF );
	}
#else
	{
		// We can do the half space ASG method cheaper by assuming H is in the YZ plane.
		float SharpnessX = LightSG.Sharpness * 2 * Pow2( NoV );
		float SharpnessY = LightSG.Sharpness * 2;

		float nu = NDF.Sharpness * 0.5;

		FSphericalGaussian ConvolvedNDF;
		ConvolvedNDF.Axis = NDF.Axis;
		ConvolvedNDF.Sharpness = 2 * (nu * SharpnessY) / (nu + SharpnessY);
		ConvolvedNDF.Amplitude = NDF.Amplitude * LightSG.Amplitude;
		ConvolvedNDF.Amplitude *= PI * rsqrt( (nu + SharpnessX) * (nu + SharpnessY) );

		//float3 AxisX = normalize( cross( N, V ) );
		//ConvolvedNDF.Amplitude *= exp( -(nu * SharpnessX) / (nu + SharpnessX) * Pow2( dot( H, AxisX ) ) );

		return Evaluate( ConvolvedNDF, H );
	}
#endif
}

FShadingOcclusion ApplyBentNormal(
	in half3 CameraVector, 
	in half3 WorldNormal, 
	in float3 WorldBentNormal0, 
	in half Roughness,
	in half MaterialAO)
{
	FShadingOcclusion Out;
	Out.DiffOcclusion = MaterialAO;
	Out.SpecOcclusion = MaterialAO;
	Out.BentNormal    = WorldNormal;

#if NUM_MATERIAL_OUTPUTS_GETBENTNORMAL > 0
	Out.BentNormal = WorldBentNormal0;

	FSphericalGaussian HemisphereSG = Hemisphere_ToSphericalGaussian(WorldNormal);
	FSphericalGaussian NormalSG = ClampedCosine_ToSphericalGaussian(WorldNormal);
	FSphericalGaussian VisibleSG = BentNormalAO_ToSphericalGaussian(Out.BentNormal, Out.DiffOcclusion );
	FSphericalGaussian DiffuseSG = Mul( NormalSG, VisibleSG );
	
	float VisibleCosAngle = sqrt( 1 - Out.DiffOcclusion );

#if 1	// Mix full resolution normal with low res bent normal
	Out.BentNormal = DiffuseSG.Axis;
	//DiffOcclusion = saturate( Integral( DiffuseSG ) / Dot( NormalSG, HemisphereSG ) );
	Out.DiffOcclusion = saturate( Integral( DiffuseSG ) * 0.42276995 );
#endif

	half3 N = WorldNormal;
	half3 V = CameraVector;

	Out.SpecOcclusion  = DotSpecularSG( Roughness, N, V, VisibleSG );
	Out.SpecOcclusion /= DotSpecularSG( Roughness, N, V, HemisphereSG );

	Out.SpecOcclusion = saturate(Out.SpecOcclusion );
#endif
	return Out;
}

#include "BasePassCommonFunctions.ush"
