// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassCommonFunctions.ush: Base pass common functions definitions
=============================================================================*/

#pragma once

float3 MaterialExpressionMainDirectionalLightIlluminance()
{
	// It is not possible to fetch that information on the view because the selected forward directional light is evaluated later in LightGridInjection render thread later.
#if defined(ForwardLightStruct)
	return ForwardLightStruct.HasDirectionalLight ? ForwardLightStruct.DirectionalLightColor : float3(0.0f, 0.0f, 0.0f);
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif
}

float3 MaterialExpressionMainDirectionalLightDirection()
{
	// It is not possible to fetch that information on the view because the selected forward directional light is evaluated later in LightGridInjection render thread later.
#if defined(ForwardLightStruct)
	return ForwardLightStruct.HasDirectionalLight ? ForwardLightStruct.DirectionalLightDirection : float3(0.0f, 0.0f, 1.0f);
#else
	return float3(0.0f, 0.0f, 1.0f);
#endif
}

