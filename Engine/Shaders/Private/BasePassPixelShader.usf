// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassPixelShader.usf: Base pass pixel shader 
=============================================================================*/

#if !MATERIAL_LWC_ENABLED
#define UE_DF_FORCE_FP32_OPS 1
#endif

#include "Common.ush"

#ifndef ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define ADAPTIVE_VOLUMETRIC_SHADOW_MAP 0
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define AVSM TranslucentBasePass.AVSM
#include "HeterogeneousVolumes/HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

// Reroute SceneTexturesStruct uniform buffer references to the appropriate base pass uniform buffer
#if MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE
	#define SceneTexturesStruct TranslucentBasePass.SceneTextures
	#define EyeAdaptationStruct TranslucentBasePass
	#define SceneColorCopyTexture	TranslucentBasePass.SceneColorCopyTexture
	#define PreIntegratedGF			TranslucentBasePass.PreIntegratedGFTexture
	#if SUPPORTS_INDEPENDENT_SAMPLERS
		#define PreIntegratedGFSampler	View.SharedBilinearClampedSampler
		#define SceneColorCopySampler	View.SharedBilinearClampedSampler
	#else
		#define PreIntegratedGFSampler	TranslucentBasePass.PreIntegratedGFSampler
		#define SceneColorCopySampler	TranslucentBasePass.SceneColorCopySampler
	#endif
	#define UseBasePassSkylight TranslucentBasePass.Shared.UseBasePassSkylight
	#define SubstrateStruct TranslucentBasePass.Substrate
	#define BlueNoise TranslucentBasePass.BlueNoise
	#define LightFunctionAtlasStruct TranslucentBasePass.Shared.LightFunctionAtlas
#else
	#define EyeAdaptationStruct OpaqueBasePass
	#define UseBasePassSkylight OpaqueBasePass.Shared.UseBasePassSkylight
	#define SubstrateStruct OpaqueBasePass.Substrate
	#define LightFunctionAtlasStruct OpaqueBasePass.Shared.LightFunctionAtlas

	// Material setting to allow forward shading (including mobile) to use preintegrated GF lut for simple IBL.
	#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER || (FORWARD_SHADING && MATERIAL_USE_PREINTEGRATED_GF)
		#define PreIntegratedGF			OpaqueBasePass.PreIntegratedGFTexture

		#if SUPPORTS_INDEPENDENT_SAMPLERS
			#define PreIntegratedGFSampler	View.SharedBilinearClampedSampler
		#else
			#define PreIntegratedGFSampler	OpaqueBasePass.PreIntegratedGFSampler
		#endif
	#endif
#endif

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#define BlueNoise SingleLayerWater.BlueNoise
#endif

// Enable Substrate. This define & include need to be defined before certains includes (i.e., DBufferDecalShared which uses them internally)
#if !MATERIAL_IS_SUBSTRATE && SUBSTRATE_ENABLED
#undef SUBSTRATE_ENABLED
#define SUBSTRATE_ENABLED 0
#endif

// Is Substrate is enabled, only use DBuffer pass after the base pass for simple and single material
#if SUBSTRATE_ENABLED && SUBSTRATE_USE_DBUFFER_PASS && USE_DBUFFER && (SUBSTRATE_FASTPATH || SUBSTRATE_SINGLEPATH)
#undef USE_DBUFFER
#define USE_DBUFFER 0
#endif

#define SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT (SUPPORT_PRIMITIVE_ALPHA_HOLDOUT && MATERIAL_DOMAIN_SURFACE)

// Enable IES profile evaluation based on project settings
#if SUPPORT_IESPROFILE_ON_FORWARD_LIT_TRANSLUCENT
#define USE_IES_PROFILE 1
#endif

#ifndef COMPUTE_SHADED
#define COMPUTE_SHADED 0
#endif

#ifndef WORKGRAPH_NODE
#define WORKGRAPH_NODE 0
#endif

#if SUBSTRATE_ENABLED && COMPUTE_SHADED
#define SUBSTRATE_VRS_WRITE 1
#endif 

// For the base pass SUBSTRATE_INLINE_SHADING is defined in BasePassRendering.h
#include "SHCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "BasePassCommon.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "LightmapCommon.ush"  
#include "PlanarReflectionShared.ush"
#include "BRDF.ush"
#include "Random.ush"
#include "LightAccumulator.ush"
#include "DeferredShadingCommon.ush"
#include "VelocityCommon.ush"
#include "DBufferDecalShared.ush"
#include "ShadingModelsSampling.ush"
#include "SGGX.ush"

#include "SceneTexturesCommon.ush"
#include "SceneTextureParameters.ush"
#include "GBufferHelpers.ush"

#include "/Engine/Generated/ShaderAutogen/AutogenShaderHeaders.ush"

#define PREV_FRAME_COLOR	1
#include "SSRT/SSRTRayCast.ush"

#if NEEDS_BASEPASS_PIXEL_FOGGING || NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING
	#include "HeightFogCommon.ush"
#if PROJECT_SUPPORT_SKY_ATMOSPHERE
	#include "SkyAtmosphereCommon.ush"
#endif
#if MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING
#include "VolumetricCloudCommon.ush"
#endif
#endif

#include "ReflectionEnvironmentShared.ush"
#if PLATFORM_FORCE_SIMPLE_SKY_DIFFUSE
#define GetEffectiveSkySHDiffuse GetSkySHDiffuseSimple
#else
#define GetEffectiveSkySHDiffuse GetSkySHDiffuse
#endif

#if MATERIALBLENDING_ANY_TRANSLUCENT
#define LumenGIVolumeStruct TranslucentBasePass
// Reroute for LumenRadianceCacheInterpolation.ush
#define RadianceCacheInterpolation TranslucentBasePass
#include "Lumen/LumenTranslucencyVolumeShared.ush"

#define FrontLayerTranslucencyStruct TranslucentBasePass
#include "FrontLayerTranslucencyShared.ush"

#define bIsTranslucentHoldoutPass (TranslucentBasePass.TranslucencyPass == 1)
#else
#define bIsTranslucentHoldoutPass false
#endif

#if TRANSLUCENT_SELF_SHADOWING
	#include "ShadowProjectionCommon.ush"	 
#endif

#include "ShadingModelsMaterial.ush"
#if MATERIAL_SHADINGMODEL_HAIR || MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#include "ShadingModels.ush"
#endif

#if MATERIAL_SHADINGMODEL_HAIR
#ifndef USE_HAIR_COMPLEX_TRANSMITTANCE
#define USE_HAIR_COMPLEX_TRANSMITTANCE 0
#endif
#include "HairStrands/HairStrandsEnvironmentLightingCommon.ush"
#endif

#ifndef COMPILER_GLSL
	#define COMPILER_GLSL 0
#endif

#define FORCE_FULLY_ROUGH (MATERIAL_FULLY_ROUGH)
#define EDITOR_ALPHA2COVERAGE (USE_EDITOR_COMPOSITING && SUPPORTS_PIXEL_COVERAGE)
#define POST_PROCESS_SUBSURFACE ((MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_EYE) && USES_GBUFFER)
#define MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING (MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT || SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE )
#define OUTPUTS_COVERAGE (MATERIALBLENDING_MASKED_USING_COVERAGE && !EARLY_Z_PASS_ONLY_MATERIAL_MASKING)

#define OIT_ENABLED (PROJECT_OIT && PERMUTATION_SUPPORTS_OIT && PLATFORM_SUPPORTS_ROV && (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING))

#if OIT_ENABLED
#define OIT_IS_BASEPASS 1
#include "OITCommon.ush"
#endif

#if  MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#if SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE

	// Use mobile like simple shading
	// Remove forward lighting directional light shadow
	#define DISABLE_FORWARD_DIRECTIONAL_LIGHT_SHADOW 1

	// Change the blend mode here from opaque to pre-multiplied-alpha for simplicity
	#undef MATERIALBLENDING_ALPHACOMPOSITE
	#define MATERIALBLENDING_ALPHACOMPOSITE 1
	#undef MATERIALBLENDING_SOLID
	#define MATERIALBLENDING_SOLID 0

#endif

#include "SingleLayerWaterShading.ush"

#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#include "ThinTranslucentCommon.ush"

#if TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || FORWARD_SHADING || MATERIAL_SHADINGMODEL_SINGLELAYERWATER || SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING
// Enables applying a cloud shadow factor to lighting calculations in ForwardLightingCommon.ush.
#define ENABLE_FORWARD_CLOUD_SHADOW NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
#include "ForwardLightingCommon.ush"
#endif

#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING (MATERIAL_IS_SUBSTRATE && SUBSTRATE_ENABLED && SUBSTRATE_OPAQUE_DEFERRED)

#if SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING || MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING || SUBSTRATE_MATERIAL_EXPORT_EXECUTED
#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#endif
#if SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING
#include "/Engine/Private/Substrate/SubstrateForwardLighting.ush"
#endif
#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING || SUBSTRATE_MATERIAL_EXPORT_EXECUTED || SUBSTRATE_GBUFFER_FORMAT==0
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

#define ENABLE_LOCAL_FOG_VOLUMES_ON_OPAQUE_FORWARD  (PROJECT_SUPPORTS_LOCALFOGVOLUME && NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING && OPAQUE_NEEDS_BASEPASS_FOGGING)
#define LOCAL_FOG_VOLUME_PER_PIXEL_ON_TRANSLUCENT (PROJECT_SUPPORTS_LOCALFOGVOLUME && NEEDS_BASEPASS_PIXEL_FOGGING)
#if ENABLE_LOCAL_FOG_VOLUMES_ON_OPAQUE_FORWARD || LOCAL_FOG_VOLUME_PER_PIXEL_ON_TRANSLUCENT
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

#include "BlueNoise.ush"

#if !FORWARD_SHADING

bool ManualDepthTestEqual(float4 SvPosition, float PSDeviceZWithOffset)
{
	const float TexDeviceZWithOffset = OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(SvPosition.xy, 0)).r;

	uint PSDeviceZWithOffsetUINT  = uint(PSDeviceZWithOffset * 16777215.0f + 0.5f);	// 16777215 = 2^24 - 1
	uint TexDeviceZWithOffsetUINT = uint(TexDeviceZWithOffset * 16777215.0f + 0.5f);
	
	const bool DepthTest_Equal = OpaqueBasePass.Is24BitUnormDepthStencil ?
		(PSDeviceZWithOffsetUINT == TexDeviceZWithOffsetUINT || PSDeviceZWithOffsetUINT == (TexDeviceZWithOffsetUINT - 1)) :	// 24 bit unorm, need to test a bit more, likely due to 24unorm conversion to float when loading form the texture.
		(PSDeviceZWithOffset == TexDeviceZWithOffset); 																			// 32 bit float, 1 to 1 mapping.

	return DepthTest_Equal;
}

#endif // !FORWARD_SHADING

#if !FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING

/** Calculates lighting for translucency. */
void GetTranslucencyVolumeLighting(
	FMaterialPixelParameters MaterialParameters, 
	FPixelMaterialInputs PixelMaterialInputs,
	FBasePassInterpolantsVSToPS BasePassInterpolants, 
	FGBufferData GBuffer, 
	float IndirectIrradiance,
	out float3 DiffuseLighting,
	out float3 SpecularLighting)
{
	float4 VolumeLighting;
	DiffuseLighting = 0;
	SpecularLighting = 0;
	
	float3 LightingPositionOffset = MaterialParameters.LightingPositionOffset;

	if (TranslucencyLightingRandomPositionOffsetRadius > 0.0f)
	{
		// override offset set by VF
		// TODO: evaluate different noise functions, maybe use actual 3D Spatiotemporal Blue Noise?
		const float3 Random = float3(BlueNoiseVec2(MaterialParameters.SvPosition.xy, ResolvedView.StateFrameIndex), BlueNoiseScalar(MaterialParameters.SvPosition.xy, ResolvedView.StateFrameIndex));
		LightingPositionOffset = (Random - 0.5f) * TranslucencyLightingRandomPositionOffsetRadius;
	}

	float3 InnerVolumeUVs;
	float3 OuterVolumeUVs;
	float FinalLerpFactor;
	ComputeVolumeUVs(MaterialParameters.WorldPosition_CamRelative, LightingPositionOffset, InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

	#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL

		GetVolumeLightingDirectional(float4(BasePassInterpolants.AmbientLightingVector, 1), BasePassInterpolants.DirectionalLightingVector, MaterialParameters.WorldNormal, GBuffer.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), DiffuseLighting, VolumeLighting);

	#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

		GetVolumeLightingNonDirectional(float4(BasePassInterpolants.AmbientLightingVector, 1), GBuffer.DiffuseColor, DiffuseLighting, VolumeLighting);

	#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME
	
		float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		float3 DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, MaterialParameters.WorldNormal, GBuffer.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), DiffuseLighting, VolumeLighting);

	#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL

		float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		GetVolumeLightingNonDirectional(AmbientLightingVector, GBuffer.DiffuseColor, DiffuseLighting, VolumeLighting);

	#elif TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING

		float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		float3 DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		FTwoBandSHVectorRGB TranslucentLighting = ReconstructSHCoefficients(AmbientLightingVector, DirectionalLightingVector);
			
		GetVolumeLightingDirectional(TranslucentLighting, AmbientLightingVector.a, MaterialParameters.WorldNormal, GBuffer.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), DiffuseLighting, VolumeLighting);

		float3 ApproxSpecularDirection = GetSpecularDirectionFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

		// TODO: need more accurate transfer coefficients to approximate specular with.
		FTwoBandSHVector TransferSH = CalcDiffuseTransferSH(ApproxSpecularDirection, /*Exponent*/ 0.0f); 

		const float3 Color = max(0.0f, DotSH(TranslucentLighting, TransferSH));

		FDeferredLightData LightData = (FDeferredLightData)0;
		LightData.Color				= Color;
		LightData.FalloffExponent	= 0;
		LightData.Direction			= ApproxSpecularDirection;
		LightData.DistanceFadeMAD	= 0;
		LightData.bRadialLight		= false;
		LightData.SpecularScale		= 1.0f;
		LightData.DiffuseScale		= 0.0f;
		LightData.ShadowedBits		= 1;
		LightData.HairTransmittance	= InitHairTransmittanceData();
		LightData.SourceRadius		= 0;
		LightData.SoftSourceRadius	= 0;
		LightData.IESAtlasIndex		= -1;
		LightData.LightFunctionAtlasLightIndex = 0;
		LightData.bAffectsTranslucentLighting = true;
		LightData.ShadowMapChannelMask = 0;

		float SurfaceShadow = 1.0f;
		FDeferredLightingSplit ApproximateLighting = GetDynamicLightingSplit(MaterialParameters.WorldPosition_CamRelative, -MaterialParameters.CameraVector, GBuffer, 1, LightData, /*LightAttenuation*/ 1.0f, /*Dither*/ 0.0f, uint2(0,0), SurfaceShadow);

		SpecularLighting += ApproximateLighting.SpecularLighting.rgb;

	#endif

	#if (TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME) && TRANSLUCENT_SELF_SHADOWING

		// Only apply self shadowing if the shadow hasn't faded out completely
		if (TranslucentSelfShadow.DirectionalLightColor.a > 0)
		{
			// Determine the shadow space position
			// Apply a stable offset to the world position used for shadowing, which blurs out high frequency details in the shadowmap with many layers
			float4 HomogeneousShadowPosition = mul(float4(WSHackToFloat(GetWorldPosition(MaterialParameters)) + MaterialParameters.LightingPositionOffset * ResolvedView.TranslucencyLightingVolumeInvSize[0].w, 1), TranslucentSelfShadow.WorldToShadowMatrix);
			float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;
			float ShadowZ = 1 - HomogeneousShadowPosition.z;
			// Lookup the shadow density at the point being shaded
			float3 ShadowDensity = CalculateTranslucencyShadowingDensity(ShadowUVs, ShadowZ) / GetMaterialTranslucentMultipleScatteringExtinction();
			// Compute colored transmission based on the density that the light ray passed through
			float3 SelfShadowing = saturate(exp(-ShadowDensity * GetMaterialTranslucentSelfShadowDensityScale()));
			// Compute a second shadow gradient to add interesting information in the shadowed area of the first
			// This is a stop gap for not having self shadowing from other light sources
			float3 SelfShadowing2 = lerp(float3(1, 1, 1), saturate(exp(-ShadowDensity * GetMaterialTranslucentSelfShadowSecondDensityScale())), GetMaterialTranslucentSelfShadowSecondOpacity());
			SelfShadowing = SelfShadowing * SelfShadowing2;

			// Force unshadowed if we read outside the valid area of the shadowmap atlas
			// This can happen if the particle system's bounds don't match its visible area
			FLATTEN
			if (any(or(ShadowUVs < TranslucentSelfShadow.ShadowUVMinMax.xy, ShadowUVs > TranslucentSelfShadow.ShadowUVMinMax.zw)))
			{
				SelfShadowing = 1;
			}

			float3 BackscatteredLighting = 0;

			#if MATERIAL_SHADINGMODEL_SUBSURFACE
			if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE)
			{
				float InScatterPower = GetMaterialTranslucentBackscatteringExponent();
				// Setup a pow lobe to approximate anisotropic in-scattering near to the light direction
				float InScattering = pow(saturate(dot(TranslucentSelfShadow.DirectionalLightDirection.xyz, MaterialParameters.CameraVector)), InScatterPower);

				float4 SSData = GetMaterialSubsurfaceData(PixelMaterialInputs);
				float3 SubsurfaceColor = SSData.rgb;

				BackscatteredLighting = 
					SubsurfaceColor
					* InScattering 
					* TranslucentSelfShadow.DirectionalLightColor.rgb
					// Energy normalization, tighter lobes should be brighter
					* (InScatterPower + 2.0f) / 8.0f 
					// Mask by shadowing, exaggerated
					* SelfShadowing * SelfShadowing
					* VolumeLighting.a;
			}
			#endif

			// The volume lighting already contains the contribution of the directional light, 
			// So calculate the amount of light to remove from the volume lighting in order to apply per-pixel self shadowing
			// VolumeLighting.a stores all attenuation and opaque shadow factors
			float3 SelfShadowingCorrection = TranslucentSelfShadow.DirectionalLightColor.rgb * VolumeLighting.a * (1 - SelfShadowing);
				
			// Combine backscattering and directional light self shadowing
			DiffuseLighting = (BackscatteredLighting + GBuffer.DiffuseColor * max(VolumeLighting.rgb - SelfShadowingCorrection, 0));
		}

	#endif
}

#endif // !FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING

/** Computes sky diffuse lighting, including precomputed shadowing. */
void GetSkyLighting(FMaterialPixelParameters MaterialParameters, VTPageTableResult LightmapVTPageTableResult, bool bEvaluateBackface, float3 WorldNormal, LightmapUVType LightmapUV, uint LightmapDataIndex, float3 SkyOcclusionUV3D, out float3 OutDiffuseLighting, out float3 OutSubsurfaceLighting)
{
	OutDiffuseLighting = 0;
	OutSubsurfaceLighting = 0;

#if (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE) && PROJECT_SUPPORTS_LUMEN && !FORWARD_SHADING
	if (IsLumenTranslucencyGIEnabled())
	{
		const float3 VolumeUV = ComputeTranslucencyGIVolumeUV(MaterialParameters.AbsoluteWorldPosition, ResolvedView.WorldToClip);

		float3 DiffuseLighting = 0;
		float3 SubsurfaceLighting = 0;

		float InvFadeLength = LumenGIVolumeStruct.TranslucencyGIGridSize.z / 0.5;
		float FadeAmount = saturate((1.0 - VolumeUV.z) * InvFadeLength);

		if(FadeAmount > 0.0)
		{
			// Lumen Dynamic GI + shadowed Skylight
			FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(MaterialParameters.AbsoluteWorldPosition, ResolvedView.WorldToClip, true);

		#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

			FOneBandSHVectorRGB TranslucencyGISH1;
			TranslucencyGISH1.R.V = TranslucencyGISH.R.V.x;
			TranslucencyGISH1.G.V = TranslucencyGISH.G.V.x;
			TranslucencyGISH1.B.V = TranslucencyGISH.B.V.x;

			FOneBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH1(1);
			DiffuseLighting = max(float3(0,0,0), DotSH1(TranslucencyGISH1, DiffuseTransferSH)) / PI;

		#else
			// Diffuse convolution
			FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(WorldNormal, 1);
			DiffuseLighting = max(half3(0,0,0), DotSH(TranslucencyGISH, DiffuseTransferSH)) / PI;

			#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
			if (bEvaluateBackface)
			{
				FTwoBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH(-WorldNormal, 1);
				SubsurfaceLighting = max(half3(0,0,0), DotSH(TranslucencyGISH, SubsurfaceTransferSH)) / PI;
			}
			#endif
		#endif
		}
		
		if (FadeAmount < 1.0)
		{
			DiffuseLighting = lerp(GetEffectiveSkySHDiffuse(WorldNormal) * ResolvedView.SkyLightColor.rgb, DiffuseLighting, FadeAmount);
			#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
			if (bEvaluateBackface)
			{
				SubsurfaceLighting = lerp(GetEffectiveSkySHDiffuse(-WorldNormal) * ResolvedView.SkyLightColor.rgb, SubsurfaceLighting, FadeAmount);
			}
			#endif
		}

		OutDiffuseLighting += DiffuseLighting;
		OutSubsurfaceLighting += SubsurfaceLighting;
	}
	else
#endif
	if (UseBasePassSkylight > 0)
	{
#if ENABLE_SKY_LIGHT
	
		float SkyVisibility = 1;
		float GeometryTerm = 1;
		float3 SkyLightingNormal = WorldNormal;
	
		#if HQ_TEXTURE_LIGHTMAP || CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING || PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
			BRANCH
			if (ShouldSkyLightApplyPrecomputedBentNormalShadowing())
			{
				float3 NormalizedBentNormal = SkyLightingNormal;

				#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
			
					float3 SkyBentNormal = GetVolumetricLightmapSkyBentNormal(SkyOcclusionUV3D);
					SkyVisibility = length(SkyBentNormal);
					NormalizedBentNormal = SkyBentNormal / max(SkyVisibility, .0001f);

				#elif HQ_TEXTURE_LIGHTMAP

					// Bent normal from precomputed texture
					float4 WorldSkyBentNormalAndOcclusion = GetSkyBentNormalAndOcclusion(LightmapVTPageTableResult, ScaleLightmapUV(LightmapUV, float2(1.0f, 2.0f)), LightmapDataIndex, MaterialParameters.SvPosition.xy);
					// Renormalize as vector was quantized and compressed
					NormalizedBentNormal = normalize(WorldSkyBentNormalAndOcclusion.xyz);
					SkyVisibility = WorldSkyBentNormalAndOcclusion.w;

				#elif CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING

					// Bent normal from the indirect lighting cache - one value for the whole object
					if (View.IndirectLightingCacheShowFlag > 0.0f)
					{
						NormalizedBentNormal = IndirectLightingCache.PointSkyBentNormal.xyz;
						SkyVisibility = IndirectLightingCache.PointSkyBentNormal.w;
					}

				#endif

				#if (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE) && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL)
					// NonDirectional lighting can't depend on the normal
					SkyLightingNormal = NormalizedBentNormal;
				#else
				
					// Weight toward the material normal to increase directionality
					float BentNormalWeightFactor = 1 - (1 - SkyVisibility) * (1 - SkyVisibility);

					// We are lerping between the inputs of two lighting scenarios based on occlusion
					// In the mostly unoccluded case, evaluate sky lighting with the material normal, because it has higher detail
					// In the mostly occluded case, evaluate sky lighting with the bent normal, because it is a better representation of the incoming lighting
					// Then treat the lighting evaluated along the bent normal as an area light, so we must apply the lambert term
					SkyLightingNormal = lerp(NormalizedBentNormal, WorldNormal, BentNormalWeightFactor);

					float DotProductFactor = lerp(saturate(dot(NormalizedBentNormal, WorldNormal)), 1, BentNormalWeightFactor);
					// Account for darkening due to the geometry term
					GeometryTerm = DotProductFactor;
				#endif
			}
		#endif
			
		// Compute the preconvolved incoming lighting with the bent normal direction
		float3 DiffuseLookup = GetEffectiveSkySHDiffuse(SkyLightingNormal) * ResolvedView.SkyLightColor.rgb;

		// Apply AO to the sky diffuse
		OutDiffuseLighting += DiffuseLookup * (SkyVisibility * GeometryTerm);

		#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
		if (bEvaluateBackface)
		{
			float3 BackfaceDiffuseLookup = GetEffectiveSkySHDiffuse(-WorldNormal) * ResolvedView.SkyLightColor.rgb;
			OutSubsurfaceLighting += BackfaceDiffuseLookup * SkyVisibility;
		}
		#endif
#endif //ENABLE_SKY_LIGHT
	}
}

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define ILCSharedSampler1					View.SharedBilinearClampedSampler
	#define ILCSharedSampler2					View.SharedBilinearClampedSampler
	#define SkyAtmTransmittanceSharedSampler	View.SharedBilinearClampedSampler
	#define SkyAtmAerialPerspecSharedSampler	View.SharedBilinearClampedSampler
#else
	#define ILCSharedSampler1					IndirectLightingCache.IndirectLightingCacheTextureSampler1
	#define ILCSharedSampler2					IndirectLightingCache.IndirectLightingCacheTextureSampler2
	#define SkyAtmTransmittanceSharedSampler	View.TransmittanceLutTextureSampler
	#define SkyAtmAerialPerspecSharedSampler	View.CameraAerialPerspectiveVolumeSampler
#endif

/** Calculates indirect lighting contribution on this object from precomputed data. */
void GetPrecomputedIndirectLightingAndSkyLight(
	FMaterialPixelParameters MaterialParameters, 
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FBasePassInterpolantsVSToPS BasePassInterpolants,
	VTPageTableResult LightmapVTPageTableResult,
	bool bEvaluateBackface,
	float3 DiffuseDir,
	float3 VolumetricLightmapBrickTextureUVs,
	out float3 OutDiffuseLighting,
	out float3 OutSubsurfaceLighting,
	out float OutIndirectIrradiance)
{
	OutIndirectIrradiance = 0;
	OutDiffuseLighting = 0;
	OutSubsurfaceLighting = 0;
	LightmapUVType SkyOcclusionUV = (LightmapUVType)0;
	uint SkyOcclusionDataIndex = 0u;

	#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING

		#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

			#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL
				FOneBandSHVectorRGB IrradianceSH;
				IrradianceSH.R.V = BasePassInterpolants.VertexIndirectAmbient.x;
				IrradianceSH.G.V = BasePassInterpolants.VertexIndirectAmbient.y;
				IrradianceSH.B.V = BasePassInterpolants.VertexIndirectAmbient.z;
			#else
				FOneBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH1(VolumetricLightmapBrickTextureUVs);
			#endif

			FOneBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH1(1);
			OutDiffuseLighting = max(float3(0,0,0), DotSH1(IrradianceSH, DiffuseTransferSH)) / PI;

		#else

			#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
				FThreeBandSHVectorRGB IrradianceSH = (FThreeBandSHVectorRGB)0;
				IrradianceSH.R.V0 = BasePassInterpolants.VertexIndirectSH[0];
				IrradianceSH.G.V0 = BasePassInterpolants.VertexIndirectSH[1];
				IrradianceSH.B.V0 = BasePassInterpolants.VertexIndirectSH[2];
			#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL
				// Limit Volume Directional to SH2 for performance
				FTwoBandSHVectorRGB IrradianceSH2 = GetVolumetricLightmapSH2(VolumetricLightmapBrickTextureUVs);
				FThreeBandSHVectorRGB IrradianceSH = (FThreeBandSHVectorRGB)0;
				IrradianceSH.R.V0 = IrradianceSH2.R.V;
				IrradianceSH.G.V0 = IrradianceSH2.G.V;
				IrradianceSH.B.V0 = IrradianceSH2.B.V;
			#else
				FThreeBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH3(VolumetricLightmapBrickTextureUVs);
			#endif

			// Diffuse convolution
			FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(DiffuseDir, 1);
			OutDiffuseLighting = max(float3(0,0,0), DotSH3(IrradianceSH, DiffuseTransferSH)) / PI;

			#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
			if (bEvaluateBackface)
			{
				FThreeBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH3(-DiffuseDir, 1);
				OutSubsurfaceLighting += max(float3(0,0,0), DotSH3(IrradianceSH, SubsurfaceTransferSH)) / PI;
			}
			#endif
		#endif

		// Visualize volumetric lightmap texel positions
		//OutDiffuseLighting = frac(VolumetricLightmapBrickTextureUVs / View.VolumetricLightmapBrickTexelSize - .5f);

	// Method for movable components which want to use a volume texture of interpolated SH samples
	#elif CACHED_VOLUME_INDIRECT_LIGHTING
		if (View.IndirectLightingCacheShowFlag > 0.0f)
		{
			// Compute volume texture UVs from world position
			float3 VolumeUVs = WSHackToFloat(GetWorldPosition(MaterialParameters)) * IndirectLightingCache.IndirectLightingCachePrimitiveScale + IndirectLightingCache.IndirectLightingCachePrimitiveAdd;
			// Clamp UV to be within the valid region
			// Pixels outside of the object's bounding box would read garbage otherwise
			VolumeUVs = clamp(VolumeUVs, IndirectLightingCache.IndirectLightingCacheMinUV, IndirectLightingCache.IndirectLightingCacheMaxUV);
			float4 Vector0 = Texture3DSample(IndirectLightingCache.IndirectLightingCacheTexture0, IndirectLightingCache.IndirectLightingCacheTextureSampler0, VolumeUVs);

			// For debugging
			#define AMBIENTONLY 0
			#if AMBIENTONLY

				OutDiffuseLighting = Vector0.rgb / SHAmbientFunction() / PI;

			#else

				float4 Vector1 = Texture3DSample(IndirectLightingCache.IndirectLightingCacheTexture1, ILCSharedSampler1, VolumeUVs);
				float4 Vector2 = Texture3DSample(IndirectLightingCache.IndirectLightingCacheTexture2, ILCSharedSampler2, VolumeUVs);

				// Construct the SH environment
				FTwoBandSHVectorRGB CachedSH;
				CachedSH.R.V = float4(Vector0.x, Vector1.x, Vector2.x, Vector0.w);
				CachedSH.G.V = float4(Vector0.y, Vector1.y, Vector2.y, Vector1.w);
				CachedSH.B.V = float4(Vector0.z, Vector1.z, Vector2.z, Vector2.w);

				// Diffuse convolution
				FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(DiffuseDir, 1);
				OutDiffuseLighting = max(half3(0,0,0), DotSH(CachedSH, DiffuseTransferSH)) / PI;

				#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
				if (bEvaluateBackface)
				{
					FTwoBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH(-DiffuseDir, 1);
					OutSubsurfaceLighting += max(half3(0,0,0), DotSH(CachedSH, SubsurfaceTransferSH)) / PI;
				}
				#endif

			#endif
		}

	// Method for movable components which want to use a single interpolated SH sample
	#elif CACHED_POINT_INDIRECT_LIGHTING 
		if (View.IndirectLightingCacheShowFlag > 0.0f)
		{
		#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL

			FOneBandSHVectorRGB PointIndirectLighting;
			PointIndirectLighting.R.V = IndirectLightingCache.IndirectLightingSHCoefficients0[0].x;
			PointIndirectLighting.G.V = IndirectLightingCache.IndirectLightingSHCoefficients0[1].x;
			PointIndirectLighting.B.V = IndirectLightingCache.IndirectLightingSHCoefficients0[2].x;

			FOneBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH1(1);
			OutDiffuseLighting = DotSH1(PointIndirectLighting, DiffuseTransferSH);

			#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
			if (bEvaluateBackface)
			{
				FOneBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH1(1);
				OutSubsurfaceLighting += DotSH1(PointIndirectLighting, SubsurfaceTransferSH);
			}
			#endif

		#else

			FThreeBandSHVectorRGB PointIndirectLighting;
			PointIndirectLighting.R.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[0];
			PointIndirectLighting.R.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[0];
			PointIndirectLighting.R.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[0];

			PointIndirectLighting.G.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[1];
			PointIndirectLighting.G.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[1];
			PointIndirectLighting.G.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[1];

			PointIndirectLighting.B.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[2];
			PointIndirectLighting.B.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[2];
			PointIndirectLighting.B.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[2];

			FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(DiffuseDir, 1);
			// Compute diffuse lighting which takes the normal into account
			OutDiffuseLighting = max(half3(0,0,0), DotSH3(PointIndirectLighting, DiffuseTransferSH));

			#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
			if (bEvaluateBackface)
			{
				FThreeBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH3(-DiffuseDir, 1);
				OutSubsurfaceLighting += max(half3(0, 0, 0), DotSH3(PointIndirectLighting, SubsurfaceTransferSH));
			}
			#endif
		
		#endif
		}

	// High quality texture lightmaps
	#elif HQ_TEXTURE_LIGHTMAP
		LightmapUVType LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
		SkyOcclusionUV = LightmapUV0;
		SkyOcclusionDataIndex = LightmapDataIndex;
		GetLightMapColorHQ(LightmapVTPageTableResult, LightmapUV0, LightmapUV1, LightmapDataIndex, DiffuseDir, MaterialParameters.SvPosition.xy, bEvaluateBackface, OutDiffuseLighting, OutSubsurfaceLighting);

	// Low quality texture lightmaps
	#elif LQ_TEXTURE_LIGHTMAP
		LightmapUVType LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
		GetLightMapColorLQ(LightmapVTPageTableResult, LightmapUV0, LightmapUV1, LightmapDataIndex, DiffuseDir, bEvaluateBackface, OutDiffuseLighting, OutSubsurfaceLighting);

	#endif

	// Apply indirect lighting scale while we have only accumulated lightmaps
	OutDiffuseLighting *= View.PrecomputedIndirectLightingColorScale;
	OutSubsurfaceLighting *= View.PrecomputedIndirectLightingColorScale;

	float3 SkyDiffuseLighting;
	float3 SkySubsurfaceLighting;
	GetSkyLighting(MaterialParameters, LightmapVTPageTableResult, bEvaluateBackface, DiffuseDir, SkyOcclusionUV, SkyOcclusionDataIndex, VolumetricLightmapBrickTextureUVs, SkyDiffuseLighting, SkySubsurfaceLighting);

	OutSubsurfaceLighting += SkySubsurfaceLighting;

	// Sky lighting must contribute to IndirectIrradiance for ReflectionEnvironment lightmap mixing
	OutDiffuseLighting += SkyDiffuseLighting;

	#if HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP || CACHED_VOLUME_INDIRECT_LIGHTING || CACHED_POINT_INDIRECT_LIGHTING || PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
		OutIndirectIrradiance = Luminance(OutDiffuseLighting);
	#endif
}

#if EDITOR_ALPHA2COVERAGE != 0
uint CustomAlpha2Coverage(inout float4 InOutColor)
{
	uint MaskedCoverage = 0xff;

	MaskedCoverage = 0;

	uint EnabledSampleCount = 1;

	// todo: support non 4xMSAA as well

	// conservatively on but can be 0 if the opacity is too low
	if(InOutColor.a > 0.01f) { MaskedCoverage |= 0x1; }
	if(InOutColor.a > 0.25f) { MaskedCoverage |= 0x2; ++EnabledSampleCount; }
	if(InOutColor.a > 0.50f) { MaskedCoverage |= 0x4; ++EnabledSampleCount; }
	if(InOutColor.a > 0.75f) { MaskedCoverage |= 0x8; ++EnabledSampleCount; }

	// renormalize to make this sample the correct weight
	InOutColor *= (float)View.NumSceneColorMSAASamples / EnabledSampleCount;

	return MaskedCoverage;
}
#endif

void ApplyPixelDepthOffsetForBasePass(inout FMaterialPixelParameters MaterialParameters, FPixelMaterialInputs PixelMaterialInputs, inout FBasePassInterpolantsVSToPS BasePassInterpolants, out float OutDepth)
{
	float PixelDepthOffset = ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);

#if WRITES_VELOCITY_TO_GBUFFER && !IS_NANITE_PASS
	if(IsOrthoProjection(ResolvedView))
	{
		BasePassInterpolants.VelocityPrevScreenPosition.z += PixelDepthOffset;
	}
	else
	{
		BasePassInterpolants.VelocityPrevScreenPosition.w += PixelDepthOffset;
	}
#endif
}

#if SUBSTRATE_ENABLED

uint GetDiffuseIndirectSampleOcclusion(FSharedLocalBases SharedLocalBases, float3 V, float2 SvPosition, float MaterialAO)
{
	uint DiffuseIndirectSampleOcclusion = 0;
#if SUBSTRATE_INLINE_SHADING && GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION && !MATERIAL_SHADINGMODEL_UNLIT
	if (MaterialAO < 1.0f)
	{
		uint OcclusionMask = 0x0;
		// We must normalize each normal and tangent to avoid non normalised vectors due to per vertex interpolation or texture filtering,
		// for the deferred (our packing relies on normalized normal) and forward (normals are going to be used as-is from registers) paths.
		UNROLL
		for (uint i = 0; i < SharedLocalBases.Count; ++i)
		{
			const float3 WorldNormal = SharedLocalBases.Normals[i];
			const float3 BentNormal = WorldNormal; // SUBSTRATE_TODO: bent normal support - GetBentNormal(MaterialParameters)
			const FSphericalGaussian HemisphereSG = Hemisphere_ToSphericalGaussian(WorldNormal);
			const FSphericalGaussian VisibleSG = BentNormalAO_ToSphericalGaussian(BentNormal, MaterialAO);
			const float VisibilityThreshold = InterleavedGradientNoise(SvPosition, View.StateFrameIndexMod8);

			for (uint TracingRayIndex = 0; TracingRayIndex < INDIRECT_SAMPLE_COUNT; TracingRayIndex++)
			{
				const float4 E = ComputeIndirectLightingSampleE(SvPosition, TracingRayIndex, INDIRECT_SAMPLE_COUNT);
				const FBxDFSample BxDFSample = SampleDiffuseBxDF(WorldNormal, E);

				// Integrate what is visible over the maximum visibility for the normal.
				float LVisibility = saturate(Evaluate(VisibleSG, BxDFSample.L) / Evaluate(HemisphereSG, BxDFSample.L));
				bool bIsBentNormalOccluded = LVisibility < VisibilityThreshold;

				OcclusionMask |= bIsBentNormalOccluded ? (1u << TracingRayIndex) : 0u;
			}
		}

		DiffuseIndirectSampleOcclusion = OcclusionMask;
	}
#endif
	return DiffuseIndirectSampleOcclusion;
}

#if SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT
void ApplyAlphaHoldOutToSubstrateOpaque(inout FSubstratePixelHeader SubstratePixelHeader, inout FSubstrateData SubstrateData)
{
	// Set a default null material to appear black. Alpha will remain correct.
	SubstrateData = GetInitialisedSubstrateData();
	SubstratePixelHeader = InitialiseSubstratePixelHeader();
}
#endif

#else // SUBSTRATE_ENABLED
uint GetDiffuseIndirectSampleOcclusion(FGBufferData GBuffer, float3 V, float3 WorldNormal, float3 WorldBentNormal, float2 SvPosition, float MaterialAO)
{
	uint DiffuseIndirectSampleOcclusion = 0;
#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION && !MATERIAL_SHADINGMODEL_UNLIT
	if (MaterialAO < 1.0f)
	{
		FSphericalGaussian HemisphereSG = Hemisphere_ToSphericalGaussian(WorldNormal);
		FSphericalGaussian VisibleSG = BentNormalAO_ToSphericalGaussian(WorldBentNormal, MaterialAO);
		float VisibilityThreshold = InterleavedGradientNoise(SvPosition, View.StateFrameIndexMod8);

		uint OcclusionMask = 0x0;

		for (uint TracingRayIndex = 0; TracingRayIndex < INDIRECT_SAMPLE_COUNT; TracingRayIndex++)
		{
			const uint TermMask = SHADING_TERM_DIFFUSE | SHADING_TERM_HAIR_R | SHADING_TERM_HAIR_TT | SHADING_TERM_HAIR_TRT;

			float4 E = ComputeIndirectLightingSampleE(SvPosition, TracingRayIndex, INDIRECT_SAMPLE_COUNT);

			FBxDFSample BxDFSample = SampleBxDF(TermMask, GBuffer, V, E);

			// Integrate what is visible over the maximum visibility for the normal.
			float LVisibility = saturate(Evaluate(VisibleSG, BxDFSample.L) / Evaluate(HemisphereSG, BxDFSample.L));
			bool bIsBentNormalOccluded = LVisibility < VisibilityThreshold;

			OcclusionMask |= bIsBentNormalOccluded ? (1u << TracingRayIndex) : 0u;
		}

		DiffuseIndirectSampleOcclusion = OcclusionMask;
	}
#endif
	return DiffuseIndirectSampleOcclusion;
}
#endif // SUBSTRATE_ENABLED


#define SELECTIVE_OUTPUT_MASK_OFFSET 4

#if USES_GBUFFER

// The selective output mask can only depend on defines, since the shadow will not export the data.
uint GetSelectiveOutputMask()
{
	uint Mask = 0;
#if MATERIAL_USES_ANISOTROPY && !IS_NANITE_PASS
	Mask |= HAS_ANISOTROPY_MASK;
#endif
#if !GBUFFER_HAS_PRECSHADOWFACTOR
	Mask |= SKIP_PRECSHADOW_MASK;
#endif
#if (GBUFFER_HAS_PRECSHADOWFACTOR && WRITES_PRECSHADOWFACTOR_ZERO)
	Mask |= ZERO_PRECSHADOW_MASK;
#endif
#if !WRITES_VELOCITY_TO_GBUFFER
	Mask |= SKIP_VELOCITY_MASK;
#endif
	return Mask;
}

void RemoveAnisotropyFromSelectiveOutputMask(inout uint InMask)
{
#if MATERIAL_USES_ANISOTROPY && !IS_NANITE_PASS
	InMask &= ~(HAS_ANISOTROPY_MASK >> SELECTIVE_OUTPUT_MASK_OFFSET);
#endif
}

#endif // USES_GBUFFER

// is called in MainPS() from PixelShaderOutputCommon.usf
void FPixelShaderInOut_MainPS(
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FBasePassInterpolantsVSToPS BasePassInterpolants,
	in FPixelShaderIn In,
	inout FPixelShaderOut Out,
	const uint EyeIndex,
	uint QuadPixelWriteMask=1,
	uint2 QuadPixelPos=0,
	bool bQuadShadingEnabled=false)
{
	// Velocity
	float4 OutVelocity = 0;
	
	// CustomData
	float4 OutGBufferD = 0;
	
	// PreShadowFactor
	float4 OutGBufferE = 0;
	
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, In.SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;

	VTPageTableResult LightmapVTPageTableResult = (VTPageTableResult)0.0f;
#if LIGHTMAP_VT_ENABLED
	{
		LightmapUVType LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
		LightmapVTPageTableResult = LightmapGetVTSampleInfo(LightmapUV0, LightmapDataIndex, In.SvPosition.xy);
	}
#endif
	
#if HQ_TEXTURE_LIGHTMAP && USES_AO_MATERIAL_MASK && !MATERIAL_SHADINGMODEL_UNLIT
	{
		LightmapUVType LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
		// Must be computed before BaseColor, Normal, etc are evaluated
		MaterialParameters.AOMaterialMask = GetAOMaterialMask(LightmapVTPageTableResult, ScaleLightmapUV(LightmapUV0, float2(1, 2)), LightmapDataIndex, In.SvPosition.xy);
	}
#endif

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(In.SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, In.SvPosition, ScreenPosition, In.bIsFrontFace, TranslatedWorldPosition, BasePassInterpolants.PixelPositionExcludingWPO);
	}
#elif IS_NANITE_PASS
	{
		float3 TranslatedWorldPosition = MaterialParameters.WorldPosition_CamRelative;
		float3 TranslatedWorldPosition_NoOffsets = MaterialParameters.WorldPosition_NoOffsets_CamRelative;
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, In.SvPosition, MaterialParameters.ScreenPosition, In.bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition_NoOffsets);
	}
#else
	{
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, In.SvPosition, In.bIsFrontFace);
	}
#endif

#if LIGHTMAP_VT_ENABLED
	// This must occur after CalcMaterialParameters(), which is required to initialize the VT feedback mechanism
	// Lightmap request is always the first VT sample in the shader
	StoreVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback, LightmapVTPageTableResult.PackedRequest, LightmapVTPageTableResult.PendingMips);
#endif

#if USE_EDITOR_COMPOSITING && (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 || MOBILE_EMULATION)
	const bool bEditorWeightedZBuffering = true;
#else
	const bool bEditorWeightedZBuffering = false;
#endif

#if OUTPUT_PIXEL_DEPTH_OFFSET
	ApplyPixelDepthOffsetForBasePass(MaterialParameters, PixelMaterialInputs, BasePassInterpolants, Out.Depth);
	
	#if APPLE_DEPTH_BIAS_HACK
	Out.Depth -= APPLE_DEPTH_BIAS_VALUE;
	#endif
#endif

	//Clip if the blend mode requires it.
#if !EARLY_Z_PASS_ONLY_MATERIAL_MASKING
	if (!bEditorWeightedZBuffering)
	{
#if MATERIALBLENDING_MASKED_USING_COVERAGE
		Out.Coverage = DiscardMaterialWithPixelCoverage(MaterialParameters, PixelMaterialInputs);
#else
		GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);
#endif
	}
#endif

	const float Dither = InterleavedGradientNoise(MaterialParameters.SvPosition.xy, View.StateFrameIndexMod8);

#if SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT
#if (MATERIALBLENDING_ALPHAHOLDOUT || SUBSTRATE_BLENDING_ALPHAHOLDOUT)
	const bool bIsHoldout = true;
#else
	const bool bIsHoldout = (GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HOLDOUT) && ResolvedView.bPrimitiveAlphaHoldoutEnabled;
#endif
#endif

#if !SUBSTRATE_ENABLED
	// Store the results in local variables and reuse instead of calling the functions multiple times.
	half3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	half  Specular = GetMaterialSpecular(PixelMaterialInputs);

	float Roughness = GetMaterialRoughness(PixelMaterialInputs);
	float Anisotropy = GetMaterialAnisotropy(PixelMaterialInputs);
	uint ShadingModel = GetMaterialShadingModel(PixelMaterialInputs);
	half Opacity = GetMaterialOpacity(PixelMaterialInputs);
#else
	half3 BaseColor = 0;
	half  Metallic = 0;
	half  Specular = 0;

	float Roughness = 0;
	float Anisotropy = 0;
	uint ShadingModel = 0;
	half Opacity = 0;
#endif

	float MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);

	// If we don't use this shading model the color should be black (don't generate shader code for unused data, don't do indirectlighting cache lighting with this color).
	float3 SubsurfaceColor = 0;
	// 0..1, SubsurfaceProfileId = int(x * 255)
	float SubsurfaceProfile = 0;
#if !SUBSTRATE_ENABLED
#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN || MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || MATERIAL_SHADINGMODEL_CLOTH || MATERIAL_SHADINGMODEL_EYE
	if (ShadingModel == SHADINGMODELID_SUBSURFACE || ShadingModel == SHADINGMODELID_PREINTEGRATED_SKIN || ShadingModel == SHADINGMODELID_SUBSURFACE_PROFILE || ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE || ShadingModel == SHADINGMODELID_CLOTH || ShadingModel == SHADINGMODELID_EYE)
	{
		float4 SubsurfaceData = GetMaterialSubsurfaceData(PixelMaterialInputs);

		if (false) // Dummy if to make the ifdef logic play nicely
		{
		}
#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
		else if (ShadingModel == SHADINGMODELID_SUBSURFACE || ShadingModel == SHADINGMODELID_PREINTEGRATED_SKIN || ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE)
		{
			SubsurfaceColor = SubsurfaceData.rgb * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
		}
#endif
#if MATERIAL_SHADINGMODEL_CLOTH
		else if (ShadingModel == SHADINGMODELID_CLOTH)
		{
			SubsurfaceColor = SubsurfaceData.rgb;
		}
#endif

		SubsurfaceProfile = SubsurfaceData.a;
	}
#endif
#endif // !SUBSTRATE_ENABLED

#if SUBSTRATE_ENABLED
	// Initialise a Substrate header with normal in registers
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
#if !SUBSTRATE_OPTIMIZED_UNLIT
	SubstratePixelHeader.IrradianceAO.MaterialAO = MaterialAO;
	SubstratePixelHeader.SetCastContactShadow(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW);
	SubstratePixelHeader.SetDynamicIndirectShadowCasterRepresentation(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION);
	SubstratePixelHeader.SetIsFirstPerson(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_IS_FIRST_PERSON);
#endif 
#endif

#if SUBSTRATE_ENABLED && SUBSTRATE_INLINE_SINGLELAYERWATER
	SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_SLWATER);

	// Override GBuffer data with Substrate SLW water BSDF to run forward shadfing code.
	// SUBSTRATE_TODO: run the shading through a Substrate path? (by adding a special BSDF?)
	SubstratePixelHeader.SubstrateTree.BSDFs[0].SubstrateSanitizeBSDF();
	FSubstrateBSDF SLWBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];

	BaseColor = SLW_BASECOLOR(SLWBSDF);
	Metallic = SLW_METALLIC(SLWBSDF);
	Specular = SLW_SPECULAR(SLWBSDF);
	Roughness = SLW_ROUGHNESS(SLWBSDF);
	Opacity = SLW_TOPMATERIALOPACITY(SLWBSDF);

	MaterialParameters.WorldNormal = normalize(SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(SLWBSDF)]);
	const float SLWWorldNormalSquaredLen = dot(MaterialParameters.WorldNormal, MaterialParameters.WorldNormal);
	BRANCH
	if (!IsFinite(SLWWorldNormalSquaredLen)) // A way to detect that the normalisation process was done on 0-length normal or with NaNs.
	{
		// In this case, reset to the world normal to the vertex normal in order to avoid abvious bad bloom on screen.
		MaterialParameters.WorldNormal = normalize(MaterialParameters.TangentToWorld[2]);
		#if MATERIAL_TANGENTSPACENORMAL
		MaterialParameters.WorldNormal *= MaterialParameters.TwoSidedSign;
		#endif
		SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(SLWBSDF)] = MaterialParameters.WorldNormal;
	}

	Anisotropy = 0.0f;
	ShadingModel = MATERIAL_SHADINGMODEL_SINGLELAYERWATER;
#endif

#if USE_DBUFFER && !MATERIALBLENDING_ANY_TRANSLUCENT && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	#if PC_D3D
	//Temporary workaround to avoid crashes on AMD, revert back to BRANCH
	FLATTEN
	#else
	BRANCH
	#endif
	if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER) != 0 && View.ShowDecalsMask > 0)
	{
	#if MATERIALDECALRESPONSEMASK
		// Apply decals from the DBuffer.
		uint ValidDBufferTargetMask = GetDBufferTargetMask(uint2(In.SvPosition.xy)) & MATERIALDECALRESPONSEMASK;

		BRANCH
		if (ValidDBufferTargetMask)
		{
			float2 BufferUV = SvPositionToBufferUV(In.SvPosition);
		#if SUBSTRATE_ENABLED 
			#if SUBSTRATE_INLINE_SHADING && !SUBSTRATE_OPTIMIZED_UNLIT
			const FSubstrateDBuffer SubstrateBufferData = SubstrateGetDBufferData(BufferUV, ValidDBufferTargetMask);
			ApplyDBufferData(SubstrateBufferData, SubstratePixelHeader, SubstrateData);
			#endif
		#else
			FDBufferData DBufferData = GetDBufferData(BufferUV, ValidDBufferTargetMask);
			ApplyDBufferData(DBufferData, MaterialParameters.WorldNormal, SubsurfaceColor, Roughness, BaseColor, Metallic, Specular);
		#endif // SUBSTRATE_ENABLED
		}
	#endif
	}
#endif

	const float BaseMaterialCoverageOverWater = Opacity;
	const float WaterVisibility = 1.0 - BaseMaterialCoverageOverWater;

	float3 VolumetricLightmapBrickTextureUVs;

#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
	VolumetricLightmapBrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WSHackToFloat(GetWorldPosition(MaterialParameters)));
#endif

	FGBufferData GBuffer = (FGBufferData)0;

	GBuffer.GBufferAO = MaterialAO;
	GBuffer.PerObjectGBufferData = GetPrimitive_PerObjectGBufferData(MaterialParameters.PrimitiveId);
	GBuffer.Depth = MaterialParameters.ScreenPosition.w;
	GBuffer.PrecomputedShadowFactors = GetPrecomputedShadowMasks(LightmapVTPageTableResult, Interpolants, MaterialParameters, VolumetricLightmapBrickTextureUVs);

#if !SUBSTRATE_ENABLED || SUBSTRATE_INLINE_SINGLELAYERWATER
	// Use GBuffer.ShadingModelID after SetGBufferForShadingModel(..) because the ShadingModel input might not be the same as the output
	SetGBufferForShadingModel(
		GBuffer,
		MaterialParameters,
		PixelMaterialInputs,
		Opacity,
		BaseColor,
		Metallic,
		Specular,
		Roughness,
		Anisotropy,
		SubsurfaceColor,
		SubsurfaceProfile,
		Dither,
		ShadingModel
		);
#endif // !SUBSTRATE_ENABLED

	// Static shadow mask
#if SUBSTRATE_ENABLED && !SUBSTRATE_OPTIMIZED_UNLIT && GBUFFER_HAS_PRECSHADOWFACTOR
	{
		// Encode shadow mask only if the shadow mask is entirely non-zero and non-one
	#if WRITES_PRECSHADOWFACTOR_ZERO
		SubstratePixelHeader.SetHasPrecShadowMask(false);
		SubstratePixelHeader.SetZeroPrecShadowMask(true);
	#else
	  #if ALLOW_STATIC_LIGHTING
		const bool bAllZero = all(GBuffer.PrecomputedShadowFactors == 0);
		const bool bAllOne  = all(GBuffer.PrecomputedShadowFactors == 1);
		if (!bAllZero && !bAllOne)
		{
			SubstratePixelHeader.SetHasPrecShadowMask(true);
		}
		else if (bAllZero)
		{
			SubstratePixelHeader.SetHasPrecShadowMask(false);
			SubstratePixelHeader.SetZeroPrecShadowMask(true);
		}
		else if (bAllOne)
	  #endif
		{
			SubstratePixelHeader.SetHasPrecShadowMask(false);
			SubstratePixelHeader.SetZeroPrecShadowMask(false);
		}
	#endif
	}
#endif

#if USES_GBUFFER
	// This requires cleanup. Shader code that uses GBuffer.SelectiveOutputMask expects the outputmask to be in
	// bits [4:7], but it gets packed as bits [0:3] in the flexible gbuffer since we might move it around.
	GBuffer.SelectiveOutputMask = GetSelectiveOutputMask() >> SELECTIVE_OUTPUT_MASK_OFFSET;
#if !ALLOW_STATIC_LIGHTING
	// ZERO_PRECSHADOW_MASK is not used when !ALLOW_STATIC_LIGHTING, so we can alias the bit with bIsFirstPerson.
	if (GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_IS_FIRST_PERSON)
	{
		GBuffer.SelectiveOutputMask |= IS_FIRST_PERSON_MASK >> SELECTIVE_OUTPUT_MASK_OFFSET;
	}
#endif
	GBuffer.Velocity = 0;
#endif

#if !SUBSTRATE_ENABLED && USES_GBUFFER && SUBSTRATE_GBUFFER_FORMAT==0
	if (!LegacyShadingModelSupportsAnisotropy(GBuffer.ShadingModelID)) // GBuffer.ShadingModelID is only valid when !SUBSTRATE_ENABLED as this stage.
	{
		RemoveAnisotropyFromSelectiveOutputMask(GBuffer.SelectiveOutputMask);
	}
#endif


#if WRITES_VELOCITY_TO_GBUFFER
	BRANCH
	if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_OUTPUT_VELOCITY) != 0)
	{
		// 2d velocity, includes camera an object motion
	#if IS_NANITE_PASS
		float3 Velocity = Calculate3DVelocity(MaterialParameters.ScreenPosition, MaterialParameters.PrevScreenPosition);
	#else
		float3 Velocity = Calculate3DVelocity(MaterialParameters.ScreenPosition, BasePassInterpolants.VelocityPrevScreenPosition);
	#endif
		float TemporalResponsiveness = GetMaterialTemporalResponsiveness(MaterialParameters);	
		float4 EncodedVelocity = EncodeVelocityToTexture(Velocity, (GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_PIXEL_ANIMATION) != 0, TemporalResponsiveness);

	#if USES_GBUFFER
		GBuffer.Velocity = EncodedVelocity;
	#else
		OutVelocity = EncodedVelocity;
	#endif
	}
#endif

	const bool bChecker = CheckerFromPixelPos(MaterialParameters.SvPosition.xy);
#if !SUBSTRATE_ENABLED || SUBSTRATE_INLINE_SINGLELAYERWATER

	// So that the following code can still use DiffuseColor and SpecularColor.
	GBuffer.SpecularColor = ComputeF0(Specular, BaseColor, Metallic);

#if MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS
	const float GeometricAARoughness = GetRoughnessFromNormalCurvature(MaterialParameters);
	GBuffer.Roughness = max(GBuffer.Roughness, GeometricAARoughness);

#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		GBuffer.CustomData.y = max(GBuffer.CustomData.y, GeometricAARoughness);
	}	
#endif
#endif

#if POST_PROCESS_SUBSURFACE
	// SubsurfaceProfile applies the BaseColor in a later pass. Any lighting output in the base pass needs
	// to separate specular and diffuse lighting in a checkerboard pattern
	if (UseSubsurfaceProfile(GBuffer.ShadingModelID))
	{
		AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(BaseColor, GBuffer.SpecularColor, Specular, bChecker);
	}
#endif
	GBuffer.DiffuseColor = BaseColor - BaseColor * Metallic;

#if USE_DEVELOPMENT_SHADERS
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		GBuffer.DiffuseColor = GBuffer.DiffuseColor * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
		GBuffer.SpecularColor = GBuffer.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
	}
#endif

    #if !FORCE_FULLY_ROUGH
	if (View.RenderingReflectionCaptureMask) // Force material rendered in reflection capture to have an expanded albedo to try to be energy conservative (when specular is removed).
	#endif
	{
		EnvBRDFApproxFullyRough(GBuffer.DiffuseColor, GBuffer.SpecularColor);
		// When rendering reflection captures, GBuffer.Roughness is already forced to 1 using RoughnessOverrideParameter in GetMaterialRoughness.
	}

	float3 InputBentNormal = MaterialParameters.WorldNormal;
			
	// Clear Coat Bottom Normal
	BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
	{
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0/255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
		InputBentNormal = OctahedronToUnitVector(oct1);
	}
	
	const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, InputBentNormal, GetWorldBentNormalZero(MaterialParameters), GBuffer.Roughness, MaterialAO);

	// FIXME: ALLOW_STATIC_LIGHTING == 0 expects this to be AO
	GBuffer.GBufferAO = AOMultiBounce( Luminance( GBuffer.SpecularColor ), ShadingOcclusion.SpecOcclusion ).g;
	#if !SUBSTRATE_INLINE_SINGLELAYERWATER
	GBuffer.DiffuseIndirectSampleOcclusion = GetDiffuseIndirectSampleOcclusion(GBuffer, MaterialParameters.CameraVector, MaterialParameters.WorldNormal, GetWorldBentNormalZero(MaterialParameters), In.SvPosition.xy, MaterialAO);
	#endif
#endif // !SUBSTRATE_ENABLED

	half3 DiffuseColor = 0;
	half3 Color = 0;
	float IndirectIrradiance = 0;

	half3 ColorSeparateSpecular = 0;
	half3 ColorSeparateEmissive = 0;

	float3 DiffuseIndirectLighting = 0;
	float3 SubsurfaceIndirectLighting = 0;

	float3 SeparatedWaterMainDirLightLuminance = float3(0, 0, 0); 

#if !SUBSTRATE_ENABLED || SUBSTRATE_INLINE_SINGLELAYERWATER

	#if !MATERIAL_SHADINGMODEL_UNLIT

		float3 DiffuseDir = ShadingOcclusion.BentNormal;
		float3 DiffuseColorForIndirect = GBuffer.DiffuseColor;

		#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
		if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
		{
			// Add subsurface energy to diffuse
			//@todo - better subsurface handling for these shading models with skylight and precomputed GI
			DiffuseColorForIndirect += SubsurfaceColor;
		}
		#endif

		#if MATERIAL_SHADINGMODEL_CLOTH
		if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
		{
			DiffuseColorForIndirect += SubsurfaceColor * saturate(GetMaterialCustomData0(PixelMaterialInputs));
		}
		#endif

		#if MATERIAL_SHADINGMODEL_HAIR
		if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			#if USE_HAIR_COMPLEX_TRANSMITTANCE
			GBuffer.CustomData.a = 1.f / 255.f;
			#endif

			const float3 N = MaterialParameters.WorldNormal;
			const float3 V = MaterialParameters.CameraVector;
			DiffuseColorForIndirect = EvaluateEnvHair(GBuffer, V, N, DiffuseDir);
		}
		#endif

		const bool bEvaluateBackface = GetShadingModelRequiresBackfaceLighting(GBuffer.ShadingModelID);
		GetPrecomputedIndirectLightingAndSkyLight(MaterialParameters, Interpolants, BasePassInterpolants, LightmapVTPageTableResult, bEvaluateBackface, DiffuseDir, VolumetricLightmapBrickTextureUVs, DiffuseIndirectLighting, SubsurfaceIndirectLighting, IndirectIrradiance);

		float IndirectOcclusion = 1.0f;
		float2 NearestResolvedDepthScreenUV = 0;
		float DirectionalLightShadow = 1.0f;
		float DirectionalLightCloudShadow = 1.0f;

		#if FORWARD_SHADING && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED)
			float2 NDC = MaterialParameters.ScreenPosition.xy / MaterialParameters.ScreenPosition.w;
			float2 ScreenUV = NDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
			NearestResolvedDepthScreenUV = CalculateNearestResolvedDepthScreenUV(ScreenUV, MaterialParameters.ScreenPosition.w);

			IndirectOcclusion = GetIndirectOcclusion(NearestResolvedDepthScreenUV, HasDynamicIndirectShadowCasterRepresentation(GBuffer));
			DiffuseIndirectLighting *= IndirectOcclusion;
			SubsurfaceIndirectLighting *= IndirectOcclusion;
			IndirectIrradiance *= IndirectOcclusion;
		#endif

		DiffuseColor += (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColor) * AOMultiBounce( GBuffer.BaseColor, ShadingOcclusion.DiffOcclusion );

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
		// Fade out diffuse as this will be handled by the single scattering lighting in water material.
		// We do this after the just above GetPrecomputedIndirectLightingAndSkyLight to keep ambiant lighting avialable.
		// We also keep the SpecularColor for sun/water interactions.
		GBuffer.DiffuseColor *= BaseMaterialCoverageOverWater;
		DiffuseColor *= BaseMaterialCoverageOverWater;
#endif

		#if TRANSLUCENCY_PERVERTEX_FORWARD_SHADING

			Color += BasePassInterpolants.VertexDiffuseLighting * GBuffer.DiffuseColor;

		#elif FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME || MATERIAL_SHADINGMODEL_SINGLELAYERWATER
			uint GridIndex = 0;

			#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
				GridIndex = ComputeLightGridCellIndex((uint2)((MaterialParameters.SvPosition.xy - ResolvedView.ViewRectMin.xy) * View.LightProbeSizeRatioAndInvSizeRatio.zw), MaterialParameters.SvPosition.w, EyeIndex);

				#if NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
					DirectionalLightCloudShadow = BasePassInterpolants.VertexCloudShadow;
				#endif

				#if FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || MATERIAL_SHADINGMODEL_SINGLELAYERWATER

					// In some cases (eg: MegaLights) we shade SURFACE_FORWARDSHADING materials using Translucency Lighting Volume.
					// Pixels using front layer direct lighting still need to do per-pixel lighting evaluation here to handle non-MegaLights.
					bool bUseTranslucencyLightingVolume = TranslucencyLightingForwardUsingVolume;
					bool bUseTranslucencyLightingVolumeSpecular = TranslucencyLightingForwardUsingVolume;

					#if TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
					#if PROJECT_SUPPORTS_MEGALIGHTS
					const bool bUseFrontLayerDirectLighting = UseFrontLayerDirectLighting(MaterialParameters.ViewBufferUV, MaterialParameters.ScreenPosition.w, MATERIAL_TRANSLUCENT_PASS_AFTERMOTIONBLUR);
					if (bUseFrontLayerDirectLighting)
					{
						float3 FrontLayerDirectLighting = SampleFrontLayerDirectLighting(MaterialParameters.ViewBufferUV);
						#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
							// FrontLayerDirectLighting already handles combining diffuse/specular for thin translucent
							// so here we just add it to ColorSeparateSpecular so TopMaterialCoverage isn't applied twice
							ColorSeparateSpecular += FrontLayerDirectLighting;
						#else
							Color += FrontLayerDirectLighting;
						#endif

						bUseTranslucencyLightingVolume = FrontLayerTranslucencyStruct.SpecularOnly > 0;
						bUseTranslucencyLightingVolumeSpecular = false;
					}
					#else // !PROJECT_SUPPORTS_MEGALIGHTS
					const bool bUseFrontLayerDirectLighting = false;
					#endif // !PROJECT_SUPPORTS_MEGALIGHTS

					if (bUseTranslucencyLightingVolume)
					{
						float3 TLVDiffuseLighting;
						float3 TLVSpecularLighting;
						GetTranslucencyVolumeLighting(MaterialParameters, PixelMaterialInputs, BasePassInterpolants, GBuffer, IndirectIrradiance, TLVDiffuseLighting, TLVSpecularLighting);
					
						#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
							DiffuseColor += TLVDiffuseLighting;
							if (bUseTranslucencyLightingVolumeSpecular)
							{
								ColorSeparateSpecular += TLVSpecularLighting;
							}
						#else
							Color += TLVDiffuseLighting;
							if (bUseTranslucencyLightingVolumeSpecular)
							{
								Color += TLVSpecularLighting;
							}
						#endif
					}
					else
					#endif // TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
					{
						float3 DirectionalLightAtmosphereTransmittance = 1.0f;

						#if NEEDS_BASEPASS_PIXEL_FOGGING && PROJECT_SUPPORT_SKY_ATMOSPHERE
						const uint LightIndex = 0;
						if (ResolvedView.AtmosphereLightDiscCosHalfApexAngle_PPTrans[LightIndex].y > 0.0f)
						{
							// Only when using forward shading, we can evaluate per pixel atmosphere transmittance.
							const float3 PlanetCenterToTranslatedWorldPos = (GetTranslatedWorldPosition(MaterialParameters) - ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;
							DirectionalLightAtmosphereTransmittance = GetAtmosphereTransmittance(
								PlanetCenterToTranslatedWorldPos, ResolvedView.AtmosphereLightDirection[LightIndex].xyz, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
								View.TransmittanceLutTexture, SkyAtmTransmittanceSharedSampler);
						}
						#endif // NEEDS_BASEPASS_PIXEL_FOGGING && PROJECT_SUPPORT_SKY_ATMOSPHERE

						#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
						const bool bSkipMainDirLightVirtualShadowMapEvaluation = SingleLayerWater.bMainDirectionalLightVSMFiltering;
						const bool bSeparateWaterMainDirLightLuminance = (SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT > 0) && SingleLayerWater.bSeparateMainDirLightLuminance;
						#else
						const bool bSkipMainDirLightVirtualShadowMapEvaluation = false;
						const bool bSeparateWaterMainDirLightLuminance = false;
						#endif
		
						#if TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
						const bool bExcludeMegaLights = bUseFrontLayerDirectLighting;
						#else
						const bool bExcludeMegaLights = false;
						#endif

						FDeferredLightingSplit ForwardDirectLighting = GetForwardDirectLightingSplit(
							In.SvPosition.xy,
							GridIndex, MaterialParameters.WorldPosition_CamRelative, MaterialParameters.CameraVector, GBuffer, NearestResolvedDepthScreenUV, MaterialParameters.PrimitiveId, EyeIndex, bExcludeMegaLights, Dither,
							DirectionalLightCloudShadow, DirectionalLightAtmosphereTransmittance, DirectionalLightShadow,
							bSeparateWaterMainDirLightLuminance, SeparatedWaterMainDirLightLuminance,
							bSkipMainDirLightVirtualShadowMapEvaluation);
					
						#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
							DiffuseColor += ForwardDirectLighting.DiffuseLighting.rgb;
							ColorSeparateSpecular += ForwardDirectLighting.SpecularLighting.rgb;
						#else
							Color += ForwardDirectLighting.DiffuseLighting.rgb;
							Color += ForwardDirectLighting.SpecularLighting.rgb;
						#endif
					}
				#endif
			#endif
				// No IBL for water in deferred: that is skipped because it is done in the water composite pass. It should however be applied when using forward shading in order to get reflection without the water composite pass.
			#if !(MATERIAL_SINGLE_SHADINGMODEL && MATERIAL_SHADINGMODEL_HAIR) && (!MATERIAL_SHADINGMODEL_SINGLELAYERWATER || FORWARD_SHADING)
				if (GBuffer.ShadingModelID != SHADINGMODELID_HAIR)
				{
					int SingleCaptureIndex = GetPrimitiveData(MaterialParameters).SingleCaptureIndex;

					half3 ReflectionColor = GetImageBasedReflectionLighting(MaterialParameters, GBuffer.Roughness, GBuffer.SpecularColor, IndirectIrradiance, GridIndex, SingleCaptureIndex)
						* IndirectOcclusion
						* AOMultiBounce(GBuffer.SpecularColor, ShadingOcclusion.SpecOcclusion);

					#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
						ColorSeparateSpecular += ReflectionColor;
					#else
						Color += ReflectionColor;
					#endif
				}
			#endif
		#endif
	#endif

#else // !SUBSTRATE_ENABLED

	float DirectionalLightShadow = 1.0f; 
	float DirectionalLightCloudShadow = 1.0f;
	float IndirectOcclusion = 1.0f;
					
#endif // !SUBSTRATE_ENABLED

	#if NEEDS_BASEPASS_VERTEX_FOGGING
		float4 HeightFogging = BasePassInterpolants.VertexFog;
	#elif NEEDS_BASEPASS_PIXEL_FOGGING
		float4 HeightFogging = CalculateHeightFog(MaterialParameters.WorldPosition_CamRelative, EyeIndex, ResolvedView);
		#if LOCAL_FOG_VOLUME_ON_TRANSLUCENT
		const float4 LocalFogVolumeContrib = BasePassInterpolants.VertexFog;
		HeightFogging = float4(LocalFogVolumeContrib.rgb + HeightFogging.rgb * LocalFogVolumeContrib.a, LocalFogVolumeContrib.a * HeightFogging.a);
		#endif // LOCAL_FOG_VOLUME_ON_TRANSLUCENT
	#else
		float4 HeightFogging = float4(0,0,0,1);
	#endif

	float4 Fogging = HeightFogging;

#if LOCAL_FOG_VOLUME_PER_PIXEL_ON_TRANSLUCENT
	{
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
		uint2 TilePos = clamp(uint2(In.SvPosition.xy / float(LFVTilePixelSize)), uint2(0, 0), LFVTileDataResolution - 1);
		float4 LocalFogVolumeContrib = GetLFVContribution(ResolvedView, TilePos, TranslatedWorldPosition);
		Fogging = float4(LocalFogVolumeContrib.rgb + Fogging.rgb * LocalFogVolumeContrib.a, LocalFogVolumeContrib.a * Fogging.a);
	}
#endif

#if NEEDS_BASEPASS_PIXEL_VOLUMETRIC_FOGGING
	if (FogStruct.ApplyVolumetricFog > 0) 
	{
	#if ENABLE_LOCAL_FOG_VOLUMES_ON_OPAQUE_FORWARD
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
		uint2 TilePos = clamp(uint2(In.SvPosition.xy / float(LFVTilePixelSize)), uint2(0, 0), LFVTileDataResolution - 1);
		float4 LocalFogVolumeContrib = GetLFVContribution(ResolvedView, TilePos, TranslatedWorldPosition);
		Fogging = float4(LocalFogVolumeContrib.rgb + Fogging.rgb * LocalFogVolumeContrib.a, LocalFogVolumeContrib.a * Fogging.a);
	#endif 

		float3 VolumeUV = ComputeVolumeUV(MaterialParameters.AbsoluteWorldPosition, ResolvedView.WorldToClip, ResolvedView);
		Fogging = CombineVolumetricFog(Fogging, VolumeUV, EyeIndex, GBuffer.Depth, ResolvedView);
	}
#endif

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
	float2 NDC = MaterialParameters.ScreenPosition.xy / MaterialParameters.ScreenPosition.w;
	float2 ScreenUV = NDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
	float4 HeterogeneousVolumeResult = AVSM_SampleCameraRadianceAndTransmittance4(ScreenUV, TranslatedWorldPosition, ResolvedView.TranslatedWorldCameraOrigin);
	Fogging.rgb = HeterogeneousVolumeResult.rgb + Fogging.rgb * HeterogeneousVolumeResult.a;
	Fogging.a *= HeterogeneousVolumeResult.a;
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#if NEEDS_BASEPASS_PIXEL_FOGGING
	const float OneOverPreExposure = ResolvedView.OneOverPreExposure;
	float4 NDCPosition = mul(float4(MaterialParameters.WorldPosition_CamRelative.xyz, 1.0f), ResolvedView.TranslatedWorldToClip);
#endif

#if NEEDS_BASEPASS_PIXEL_FOGGING && PROJECT_SUPPORT_SKY_ATMOSPHERE && MATERIAL_IS_SKY==0 // Do not apply aerial perpsective on sky materials
	if (ResolvedView.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f)
	{
		// Sample the aerial perspective (AP).
		Fogging = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			NDCPosition, MaterialParameters.WorldPosition_CamRelative * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, SkyAtmAerialPerspecSharedSampler,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			OneOverPreExposure, Fogging);
	}
#endif

#if NEEDS_BASEPASS_PIXEL_FOGGING && MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING

	if (TranslucentBasePass.ApplyVolumetricCloudOnTransparent > 0.0f)
	{
		Fogging = GetCloudLuminanceTransmittanceOverFog(
			NDCPosition, GetTranslatedWorldPosition(MaterialParameters), ResolvedView.TranslatedWorldCameraOrigin,
			TranslucentBasePass.VolumetricCloudColor, TranslucentBasePass.VolumetricCloudColorSampler,
			TranslucentBasePass.VolumetricCloudDepth, TranslucentBasePass.VolumetricCloudDepthSampler,
			OneOverPreExposure, Fogging, TranslucentBasePass.SoftBlendingDistanceKm, 
			TranslucentBasePass.VolumetricCloudColorUVScale, TranslucentBasePass.VolumetricCloudColorUVMax);
	}

#endif

	half3 Emissive = 0;
#if !SUBSTRATE_ENABLED

	// Volume lighting for lit translucency
#if (MATERIAL_SHADINGMODEL_DEFAULT_LIT || MATERIAL_SHADINGMODEL_SUBSURFACE) && (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE) && !FORWARD_SHADING && !TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
	if (GBuffer.ShadingModelID == SHADINGMODELID_DEFAULT_LIT || GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE)
	{
		float3 TLVDiffuseLighting;
		float3 TLVSpecularLighting;
		GetTranslucencyVolumeLighting(MaterialParameters, PixelMaterialInputs, BasePassInterpolants, GBuffer, IndirectIrradiance, TLVDiffuseLighting, TLVSpecularLighting);
		Color += TLVDiffuseLighting;
		Color += TLVSpecularLighting;
	}
#endif

	#if !MATERIAL_SHADINGMODEL_UNLIT && USE_DEVELOPMENT_SHADERS
		float3 GBufferDiffuseColor = GBuffer.DiffuseColor;
		float3 GBufferSpecularColor = GBuffer.SpecularColor;
		EnvBRDFApproxFullyRough(GBufferDiffuseColor, GBufferSpecularColor);
		Color = lerp(Color, GBufferDiffuseColor, View.UnlitViewmodeMask);
	#endif

	Emissive = GetMaterialEmissive(PixelMaterialInputs);

#endif // !SUBSTRATE_ENABLED

#if USE_DEVELOPMENT_SHADERS
	float3 OutOfBoundsMaskLuminance = 0;
	// This feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	// The following block is disabled on Vulkan because it triggers an nvidia driver bug (UE-101609).
	#if !VULKAN_PROFILE_SM5 && !VULKAN_PROFILE_SM6
		#if METAL_SM5_PROFILE || METAL_SM6_PROFILE || SM6_PROFILE || SM5_PROFILE || VULKAN_PROFILE_SM5 || VULKAN_PROFILE_SM6
		BRANCH
		if (View.OutOfBoundsMask > 0)
		{
			FPrimitiveSceneData PrimitiveData = GetPrimitiveData(MaterialParameters);

			float3 ObjectBounds =
				float3(
					PrimitiveData.ObjectBoundsX,
					PrimitiveData.ObjectBoundsY,
					PrimitiveData.ObjectBoundsZ
				);

			if (any(abs(DFFastLocalSubtractDemote(MaterialParameters.AbsoluteWorldPosition, PrimitiveData.ObjectWorldPosition)) > ObjectBounds + 1))
			{
				// fairly cheap DF->F32, repeating every 2^16. Has seams, but that doesn't matter here
				float3 WorldPosModulo = DFFmodByPow2Demote(MaterialParameters.AbsoluteWorldPosition, 65536.0);

				float Gradient = frac(dot(WorldPosModulo, float3(.577f, .577f, .577f) / 500.0f));
				OutOfBoundsMaskLuminance = lerp(float3(1,1,0), float3(0,1,1), Gradient.xxx > .5f);
				Emissive = OutOfBoundsMaskLuminance;
				Opacity = 1;
			}
			else if (PrimitiveData.MaxWPOExtent > 0.0f)
			{
				// this highlights pixels that are close to Max WPO Distance boundary (and might be getting clamped)
				const float3 OffsetAmt = abs(MaterialParameters.WorldPosition_CamRelative - MaterialParameters.WorldPosition_NoOffsets_CamRelative);
				const float MaxOffsetDim = max3(OffsetAmt.x, OffsetAmt.y, OffsetAmt.z);
				const float Proximity = 1.0f - saturate(abs(MaxOffsetDim - PrimitiveData.MaxWPOExtent) / 0.05f);
				Emissive = Proximity * float3(1, 0 ,1);
				Opacity = sign(Proximity);
			}
		}
		#endif
	#endif
#endif

#if MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING || SUBSTRATE_TRANSLUCENT_MATERIAL || SUBSTRATE_FORWARD_SHADING || SUBSTRATE_MATERIAL_EXPORT_EXECUTED || SUBSTRATE_OPTIMIZED_UNLIT
	float3 DualBlendSurfaceLuminancePostCoverage	= 0.0f;
	float3 DualBlendSurfaceTransmittancePreCoverage = 1.0f;
	float DualBlendSurfaceCoverage					= 1.0f;
#endif

#if !SUBSTRATE_ENABLED || SUBSTRATE_INLINE_SINGLELAYERWATER

#if !POST_PROCESS_SUBSURFACE && !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
 	// For skin we need to keep them separate. We also keep them separate for thin translucent.
	// Otherwise just add them together.
	Color += DiffuseColor;
#endif

#if !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	Color += Emissive;
#endif

#endif // !SUBSTRATE_ENABLED

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER || SUBSTRATE_INLINE_SINGLELAYERWATER
	{
		const bool CameraIsUnderWater = false;	// Fade out the material contribution over to water contribution according to material opacity.
		float3 SunIlluminance = ResolvedView.DirectionalLightColor.rgb * PI;	// times PI because it is divided by PI on CPU (=luminance) and we want illuminance here. 
		float3 WaterDiffuseIndirectIlluminance = DiffuseIndirectLighting * PI;// DiffuseIndirectLighting is luminance. So we need to multiply by PI to get illuminance.
#if USE_DEVELOPMENT_SHADERS
		SunIlluminance = lerp(SunIlluminance, 0.0f, View.UnlitViewmodeMask);
		WaterDiffuseIndirectIlluminance = lerp(WaterDiffuseIndirectIlluminance, PI, View.UnlitViewmodeMask);
#endif
		const bool bSeparateWaterMainDirLightLuminance = (SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT > 0) && SingleLayerWater.bSeparateMainDirLightLuminance;

		// Evaluate Fresnel effect
		const float3 N = MaterialParameters.WorldNormal;
		const float3 V = MaterialParameters.CameraVector;
		const float3 EnvBrdf = EnvBRDF(GBuffer.SpecularColor, GBuffer.Roughness, max(0.0, dot(N, V)));

#if SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE
		const float4 NullDistortionParams = 1.0f;
		WaterVolumeLightingOutput WaterLighting = EvaluateWaterVolumeLighting(
			MaterialParameters, PixelMaterialInputs, ResolvedView,
			DirectionalLightShadow * DirectionalLightCloudShadow,
			SingleLayerWater.SceneDepthWithoutSingleLayerWaterTexture, SingleLayerWaterSceneDepthSampler, // Scene depth texture
			SingleLayerWater.SceneWithoutSingleLayerWaterTextureSize,
			SingleLayerWater.SceneWithoutSingleLayerWaterInvTextureSize,
			Specular, NullDistortionParams,
			SunIlluminance, WaterDiffuseIndirectIlluminance, EnvBrdf,
			CameraIsUnderWater, WaterVisibility, EyeIndex,
			bSeparateWaterMainDirLightLuminance, SeparatedWaterMainDirLightLuminance);

		// Add water luminance contribution
		Color += WaterLighting.Luminance;
		// Combine top layer opacity with water transmittance (grey scale)
		Opacity = 1.0 - ((1.0 - Opacity) * dot(WaterLighting.WaterToSceneToLightTransmittance, float3(1.0 / 3.0, 1.0 / 3.0, 1.0 / 3.0)));
#else
		const float4 SLWRefractionMinMaxUV = GetSingleLayerWaterRefractionUVMinMax(SingleLayerWater.SceneWithoutSingleLayerWaterTextureSize, SingleLayerWater.SceneWithoutSingleLayerWaterInvTextureSize);

		Color += EvaluateWaterVolumeLighting(
			MaterialParameters, PixelMaterialInputs, ResolvedView,
			DirectionalLightShadow * DirectionalLightCloudShadow,
			SingleLayerWater.SceneDepthWithoutSingleLayerWaterTexture, SingleLayerWaterSceneDepthSampler,
			SingleLayerWater.SceneWithoutSingleLayerWaterTextureSize,
			SingleLayerWater.SceneWithoutSingleLayerWaterInvTextureSize,
			SingleLayerWater.SceneColorWithoutSingleLayerWaterTexture, SingleLayerWaterSceneColorSampler,
			SLWRefractionMinMaxUV.xy,
			SLWRefractionMinMaxUV.zw,
			SingleLayerWater.RefractionMaskTexture,
			Specular, SingleLayerWater.DistortionParams,
			SunIlluminance, WaterDiffuseIndirectIlluminance, EnvBrdf,
			CameraIsUnderWater, WaterVisibility, EyeIndex,
			bSeparateWaterMainDirLightLuminance, SeparatedWaterMainDirLightLuminance
		#if USE_LIGHT_FUNCTION_ATLAS
			, GetLocalLightFunctionCommon(SvPositionToResolvedTranslatedWorld(In.SvPosition), GetDirectionalLightData().LightFunctionAtlasLightIndex)
		#endif
			).Luminance;
#endif
	}
#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && !SUBSTRATE_ENABLED
	{
		AccumulateThinTranslucentModel(
								DualBlendSurfaceLuminancePostCoverage,
								DualBlendSurfaceTransmittancePreCoverage,
								DualBlendSurfaceCoverage,
								MaterialParameters,
								GBuffer,
								DiffuseColor,
								ColorSeparateSpecular,
								Emissive,
								Opacity);
		Color = 0;
		Opacity = 1.0f;
	}
#endif // MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT

#if SUBSTRATE_MATERIAL_EXPORT_EXECUTED
	float4 SubstrateMaterialExport = float4(0, 0, 0, 0);
#endif

	bool bSubstrateSubsurfaceEnable = false;
#if SUBSTRATE_ENABLED && !SUBSTRATE_OPTIMIZED_UNLIT

#if SUBSTRATE_INLINE_SHADING
	SanitizeAndNormalizeSharedLocalBases(SubstratePixelHeader.SharedLocalBases, MaterialParameters.TangentToWorld);
#endif

	// Export non-unlit materials (opaque or translucent)
#if SUBSTRATE_MATERIAL_EXPORT_EXECUTED


	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*SubstrateStruct.bRoughDiffuse*/, SubstrateStruct.PeelLayersAboveDepth, SubstrateStruct.bRoughnessTracking);

	const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
	// Align V with N to not have any view dependent effect when exporting material data.
	const float3 FakeV = SurfaceWorldNormal;
	FExportResult Export = SubstrateMaterialExportOut(
		Settings,
		SubstratePixelHeader,
		SubstrateData,
		FakeV,
		SurfaceWorldNormal,
		MaterialParameters.WorldPosition_CamRelative,
		0.0f /*MobileShadingPathCurvature*/);

	#if SUBSTRATE_MATERIAL_EXPORT_FROM_OPAQUE
		#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
			SubstrateMaterialExport = float4(Export.BaseColorPostCoverage, 1.0);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR
			SubstrateMaterialExport = float4(Export.BaseColor, 1.0);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
			SubstrateMaterialExport = float4(Export.WorldNormal * 0.5 + 0.5, 1.0);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
			SubstrateMaterialExport = float4(Export.EmissiveLuminance, 1.0);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_SPECULAR
			SubstrateMaterialExport = float4(Export.Specular.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_ROUGHNESS
			SubstrateMaterialExport = float4(Export.Roughness.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_ANISOTROPY
			SubstrateMaterialExport = float4(Export.Anisotropy.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_METALLIC
			SubstrateMaterialExport = float4(Export.Metallic.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
			SubstrateMaterialExport = float4(Export.TransmittancePreCoverage * Export.Coverage + (1.0f - Export.Coverage), 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_OPACITY
			SubstrateMaterialExport = float4(Export.Coverage.xxx, 1.0f);
			#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE
			SubstrateMaterialExport = float4(GetMaterialOpacity(PixelMaterialInputs).xxx, 1.0f);
			#endif
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_OPACITYMASK
			SubstrateMaterialExport = float4(PixelMaterialInputs.OpacityMask.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_CUSTOMDATA0
			SubstrateMaterialExport = Export.CustomData0;
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_CUSTOMDATA1
			SubstrateMaterialExport = Export.CustomData1;
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_SUBSURFACECOLOR
			SubstrateMaterialExport = float4(Export.SubsurfaceColor, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TANGENT
			SubstrateMaterialExport = float4(Export.WorldTangent * 0.5 + 0.5, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_SHADINGMODEL
			SubstrateMaterialExport = float4(Export.ShadingModelID.xxx, 1.0f);
		#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
			SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
		#endif
	#elif SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT
		#if SUBSTRATE_USES_CONVERSION_FROM_LEGACY
			#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
				SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
				SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
				SubstrateMaterialExport = float4(Export.EmissiveLuminance, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
				#if SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODE == SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODULATE
					SubstrateMaterialExport = float4(Export.BaseColorPostCoverage, 1.0f);
				#else
					SubstrateMaterialExport = float4(Export.BaseColorPostCoverage + saturate(1.0f - Export.Coverage), 1.0f);
				#endif
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
				SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
			#endif
		#else
			#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
				SubstrateMaterialExport = float4(Export.BaseColorPostCoverage, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
				SubstrateMaterialExport = float4(Export.WorldNormal * 0.5 + 0.5, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
				SubstrateMaterialExport = float4(Export.EmissiveLuminance, 1.0);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
				SubstrateMaterialExport = float4(Export.TransmittancePreCoverage * Coverage + (1.0f - Coverage), 1.0f);
			#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
				SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
			#endif
		#endif
	#else
	#error Unhandled SUBSTRATE_MATERIAL_EXPORT
	#endif

#else // SUBSTRATE_MATERIAL_EXPORT_EXECUTED

#if SUBSTRATE_OPAQUE_DEFERRED

#if SUBSTRATE_INLINE_SINGLELAYERWATER==0
	// Need to reset color to make sure Substrate material are only lit using Substrate lighting passes.
	// Except for water doing some specialised and simplified lighting during the base pass
	Color = 0;
#endif

	{
		SubstratePixelHeader.IrradianceAO									= InitIrradianceAndOcclusion();
		SubstratePixelHeader.IrradianceAO.MaterialAO						= MaterialAO;			// This is the material AO, it will be converted to multibounce AO using ShadingOcclusion.SpecOcclusion in the PackSubstrateOut function, also accounting for bent normal.
		SubstratePixelHeader.IrradianceAO.IndirectIrradiance				= IndirectIrradiance;
		SubstratePixelHeader.IrradianceAO.DiffuseIndirectSampleOcclusion	= GetDiffuseIndirectSampleOcclusion(SubstratePixelHeader.SharedLocalBases, MaterialParameters.CameraVector, In.SvPosition.xy, SubstratePixelHeader.IrradianceAO.MaterialAO);
	}

	// We only rely on this GBuffer structure for the write out to legacy render target such as velocity or precomputed shadow factors
	const float4 PrecomputedShadowFactors	= GBuffer.PrecomputedShadowFactors;
	const uint   SelectiveOutputMask		= GBuffer.SelectiveOutputMask;
	const float4 EncodedVelocity			= GBuffer.Velocity;
	GBuffer = (FGBufferData)0;
	GBuffer.PrecomputedShadowFactors		= PrecomputedShadowFactors;
	GBuffer.SelectiveOutputMask				= SelectiveOutputMask;
	GBuffer.Velocity						= EncodedVelocity;

#if OUTPUT_PIXEL_DEPTH_OFFSET && SUBSTRATE_INLINE_SINGLELAYERWATER==0
	// When in deferred, opaque materials with pixel depth offset must execute a custom depth test in order to avoid conflicting UAV writes.
	// This is however not done for single layer water as the depth will never match. Note: that can lead overlapping SLW polys to cause flicker on screen.
	if (ManualDepthTestEqual(In.SvPosition, Out.Depth))
#endif
	{
		float3 EmissiveLuminance = 0.0f;
		float3 ScatteredPrecomputedLuminance = 0.0f;
		uint2 PixelPos = uint2(In.SvPosition.xy);
		const float3 WorldBentNormal0 = GetWorldBentNormalZero(MaterialParameters);

		FSubstrateSubsurfaceData SSSData = (FSubstrateSubsurfaceData)0;
		FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
		FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;
		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, SubstrateStruct.bRoughDiffuse, SubstrateStruct.PeelLayersAboveDepth, SubstrateStruct.bRoughnessTracking);

	#if SUBSTRATE_ADVANCED_DEBUG_ENABLED && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
		Settings.SliceStoringDebugSubstrateTreeData = all(uint2(float2(View.CursorPosition) * View.ViewResolutionFraction) == PixelPos) ? SubstrateStruct.SliceStoringDebugSubstrateTreeDataWithoutMRT : Settings.SliceStoringDebugSubstrateTreeData;
	#endif

	#if SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT
		if (bIsHoldout)
		{
			ApplyAlphaHoldOutToSubstrateOpaque(SubstratePixelHeader, SubstrateData);
		}
	#endif


#if SUBSTRATE_GBUFFER_FORMAT==0
		// GBuffer has been reset above already.

		const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
		const float3 V = MaterialParameters.CameraVector;
		FExportResult Export = SubstrateMaterialExportOut(
			Settings,
			SubstratePixelHeader,
			SubstrateData,
			V,
			SurfaceWorldNormal,
			MaterialParameters.WorldPosition_CamRelative,
			0.0f, // Curvature
			Dither
			);
		
		// Propagate some more data
		GBuffer.GBufferAO = MaterialAO;
		GBuffer.PerObjectGBufferData = GetPrimitive_PerObjectGBufferData(MaterialParameters.PrimitiveId);
		GBuffer.Depth = MaterialParameters.ScreenPosition.w;
		// GBuffer.SelectiveOutputMask is setup above.
		EmissiveLuminance = Export.EmissiveLuminance;

	#if USE_DEVELOPMENT_SHADERS
		TopLayerData.UnlitViewBaseColor = Export.UnlitViewBaseColor;
	#endif
		
		// Patch material parameter to not have to change SetGBufferForShadingModel for now
		MaterialParameters.WorldNormal = Export.WorldNormal;
		MaterialParameters.WorldTangent = Export.WorldTangent;
	#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
		MaterialParameters.WorldVertexNormal_Center = Export.WorldNormal;	// SUBSTRATE_TODO: how to evluate it for center?
	#endif
		MaterialParameters.TangentToWorld[2].xyz = Export.WorldTangent;

		// Use GBuffer.ShadingModelID after SetGBufferForShadingModel(..) because the ShadingModel input might not be the same as the output
		SetGBufferForShadingModel(
			GBuffer,
			MaterialParameters,
			PixelMaterialInputs,
			Export.Coverage,		// Opacity
			Export.BaseColor,
			Export.Metallic,
			Export.Specular,
			Export.Roughness,
			Export.Anisotropy,
			Export.SubsurfaceColor,
			Export.SubsurfaceProfileIDFloat,
			Dither,
			Export.ShadingModelID
			);

		// If a material is marked as aniso on the CPU, but actually does not have anisotropy for the current pixel, remove it.
		if (Export.Anisotropy == 0)
		{
			RemoveAnisotropyFromSelectiveOutputMask(GBuffer.SelectiveOutputMask);
		}

		GBuffer.SpecularColor = ComputeF0(GBuffer.Specular, GBuffer.BaseColor, GBuffer.Metallic);
		GBuffer.DiffuseColor = GBuffer.BaseColor - GBuffer.BaseColor * GBuffer.Metallic;

		// Set custom data from Substrate export after SetGBufferForShadingModel since it access MaterialParameters to set specific data not available with Substrate
		GBuffer.CustomData = Export.CustomData;

		GBuffer.StoredBaseColor = GBuffer.BaseColor;
		GBuffer.StoredMetallic = GBuffer.Metallic;
		GBuffer.StoredSpecular = GBuffer.Specular;

	#if MATERIAL_SHADINGMODEL_EYE
		if( GBuffer.ShadingModelID == SHADINGMODELID_EYE )
		{
			GBuffer.Metallic = 0.0;
			#if IRIS_NORMAL
			GBuffer.Specular = 0.25;
			#endif
		}
	#endif

		// Water currently takes the legacy path for forward shading, so we need to skip it here so we don't apply this term twice
		if (MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING && GBuffer.ShadingModelID != SHADINGMODELID_UNLIT && SUBSTRATE_INLINE_SINGLELAYERWATER == 0)
		{
			float3 InputBentNormal = MaterialParameters.WorldNormal;
			// Clear Coat Bottom Normal
			BRANCH if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
			{
				const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
				InputBentNormal = OctahedronToUnitVector(oct1);
			}

			const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, InputBentNormal, GetWorldBentNormalZero(MaterialParameters), GBuffer.Roughness, MaterialAO);
			float3 DiffuseDir = ShadingOcclusion.BentNormal;
			float3 DiffuseColorForIndirect = GBuffer.DiffuseColor;

			if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN || GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
			{
				DiffuseColorForIndirect += SubsurfaceColor;
			}
			if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
			{
			#if USE_HAIR_COMPLEX_TRANSMITTANCE
				GBuffer.CustomData.a = 1.f / 255.f;
			#endif

				const float3 N = MaterialParameters.WorldNormal;
				const float3 V = MaterialParameters.CameraVector;
				DiffuseColorForIndirect = EvaluateEnvHair(GBuffer, V, N, DiffuseDir);
			}

			const bool bEvaluateBackface = GetShadingModelRequiresBackfaceLighting(GBuffer.ShadingModelID);
			GetPrecomputedIndirectLightingAndSkyLight(MaterialParameters, Interpolants, BasePassInterpolants, LightmapVTPageTableResult, bEvaluateBackface, DiffuseDir, VolumetricLightmapBrickTextureUVs, DiffuseIndirectLighting, SubsurfaceIndirectLighting, IndirectIrradiance);

			float IndirectOcclusion = 1.0f;
			float2 NearestResolvedDepthScreenUV = 0;
			float DirectionalLightShadow = 1.0f;
			float DirectionalLightCloudShadow = 1.0f;

		#if FORWARD_SHADING && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED)
			float2 NDC = MaterialParameters.ScreenPosition.xy / MaterialParameters.ScreenPosition.w;
			float2 ScreenUV = NDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
			NearestResolvedDepthScreenUV = CalculateNearestResolvedDepthScreenUV(ScreenUV, MaterialParameters.ScreenPosition.w);

			IndirectOcclusion = GetIndirectOcclusion(NearestResolvedDepthScreenUV, HasDynamicIndirectShadowCasterRepresentation(GBuffer));
			DiffuseIndirectLighting *= IndirectOcclusion;
			SubsurfaceIndirectLighting *= IndirectOcclusion;
			IndirectIrradiance *= IndirectOcclusion;
		#endif

			Color += (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColor) * AOMultiBounce(GBuffer.BaseColor, ShadingOcclusion.DiffOcclusion);
		}

#else // SUBSTRATE_GBUFFER_FORMAT==0

		if (!bQuadShadingEnabled)
		{
			QuadPixelPos = PixelPos;
		}

		// Generate the Substrate material data to write out
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(ResolvedView.BufferSizeAndInvSize.xy), SubstrateStruct.MaxBytesPerPixel);
		FRWSubstrateMaterialContainer RWSubstrateMaterialContainer = InitialiseRWSubstrateMaterialContainer(SubstrateStruct.MaterialTextureArrayUAVWithoutRTs, QuadPixelWriteMask, QuadPixelPos);
		PackSubstrateOut(
			RWSubstrateMaterialContainer,
			SubstrateStruct.MaterialTextureArrayUAVWithoutRTs,
			Dither,
			Settings,
			SubstrateAddressing,
			SubstratePixelHeader, SubstrateData, MaterialParameters.CameraVector, WorldBentNormal0, bSubstrateSubsurfaceEnable, EmissiveLuminance,
			SSSData, TopLayerData, OpaqueRoughRefractionData
	#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
			,MaterialParameters
			,Interpolants
			,BasePassInterpolants
			,LightmapVTPageTableResult
			,VolumetricLightmapBrickTextureUVs
			,ScatteredPrecomputedLuminance
	#endif
		);

	#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
		// Add ScatteredPrecomputedLuminance to DiffuseColor in order to handle correctly SSS diffuse with precomputed lighting.
		// Luminance added to Color and to DiffuseColor are disjoint (emissive luminance vs. scattered precomputed luminance)
		DiffuseColor = ScatteredPrecomputedLuminance;
	#endif

		// Write out MRT data
	#if SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT != 3
	#error Substrate SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT has been update but not SubstrateOutput
	#endif
		Out.SubstrateOutput[0] = RWSubstrateMaterialContainer.MaterialRenderTargets[0];
		Out.SubstrateOutput[1] = RWSubstrateMaterialContainer.MaterialRenderTargets[1];
		Out.SubstrateOutput[2] = RWSubstrateMaterialContainer.MaterialRenderTargets[2];
		Out.SubstrateTopLayerData = SubstratePackTopLayerData(TopLayerData);
	#if SUBSTRATE_OUTPUT_ROUGH_REFRACTION
		OpaqueRoughRefractionData.OpaqueRoughRefractionEnabled = true; // As long as we output to rough refraction, we want it to be enabled for the tiling process.
		if (QuadPixelWriteMask>0)
		{
			const float3 OpaqueRefractionData = SubstratePackOpaqueRoughRefractionData(OpaqueRoughRefractionData);
			WriteDataToBuffer(SubstrateStruct.OpaqueRoughRefractionTextureUAV, OpaqueRefractionData, PixelPos, QuadPixelWriteMask, QuadPixelPos);
		}
	#endif

		// Only write SSS data if needed
		BRANCH
		if(SubstrateSubSurfaceHeaderGetIsValid(SSSData.Header) && QuadPixelWriteMask>0)
		{
			// Subsurface header
			{
				SubstrateStoreSubsurfaceHeader(SubstrateStruct.MaterialTextureArrayUAVWithoutRTs, SubstrateStruct.FirstSliceStoringSubstrateSSSDataWithoutMRT, PixelPos, SSSData.Header.Bytes, QuadPixelWriteMask, QuadPixelPos);
			}
			// Subsurface data
			if (SubstrateSubSurfaceHeaderHasExtras(SSSData.Header))
			{
				SubstrateStoreSubsurfaceExtras(SubstrateStruct.MaterialTextureArrayUAVWithoutRTs, SubstrateStruct.FirstSliceStoringSubstrateSSSDataWithoutMRT, PixelPos, SSSData.Extras.Bytes, QuadPixelWriteMask, QuadPixelPos);
			}
		}

#endif // SUBSTRATE_GBUFFER_FORMAT==0

		// Unlit view mode
	#if USE_DEVELOPMENT_SHADERS
		Color  = lerp(Color, TopLayerData.UnlitViewBaseColor, View.UnlitViewmodeMask);
		EmissiveLuminance += OutOfBoundsMaskLuminance;
	#endif

		Color += EmissiveLuminance;
	}

#endif // SUBSTRATE_OPAQUE_DEFERRED

#if (SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING) && !SUBSTRATE_INLINE_SINGLELAYERWATER && !SUBSTRATE_OPTIMIZED_UNLIT
	//FORWARD_SHADING
	if (SubstratePixelHeader.ClosureCount > 0)
	{
		float2 ScreenUV = ScreenPositionToBufferUV(MaterialParameters.ScreenPosition);

		#if defined(FORCE_FULLY_ROUGH) && FORCE_FULLY_ROUGH
		const bool bForceFullyRough = true;
		#else
		const bool bForceFullyRough = View.RenderingReflectionCaptureMask > 0;
		#endif

		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(bForceFullyRough, SubstrateStruct.bRoughDiffuse, SubstrateStruct.PeelLayersAboveDepth, SubstrateStruct.bRoughnessTracking);

		float3 Throughput = 1.0f;
		Color = SubstrateForwardLighting(
			EyeIndex,
			In.SvPosition,
			Settings,
			BasePassInterpolants,
			Interpolants,
			LightmapVTPageTableResult,
			VolumetricLightmapBrickTextureUVs,
			MaterialParameters,
			GBuffer.Depth,
			ScreenUV,
			SubstratePixelHeader,
			SubstrateData,
			DualBlendSurfaceTransmittancePreCoverage,
			DualBlendSurfaceCoverage);

	#if SUBSTRATE_TRANSLUCENT_MATERIAL
		DualBlendSurfaceLuminancePostCoverage = Color;
		Color = 0.0f;
		Opacity = 1.0f; // nullify following operation
	#elif SUBSTRATE_FORWARD_SHADING
		// Color unchanged for opaque materials
		Opacity = 1.0f;
	#else
		#error Unhandled Substrate forward shading mode
	#endif
	}
#endif // SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING

#endif // SUBSTRATE_MATERIAL_EXPORT_EXECUTED

#elif SUBSTRATE_ENABLED && SUBSTRATE_OPTIMIZED_UNLIT

	// Unlit BSDF goes through the SubstrateTree to support weighting operations. Technically, layering and mixing could also be supported.
	float3	UnlitSurfaceNormal = 0.0f;
	SubstratePixelHeader.SubstrateUpdateTreeUnlit(
		uint2(In.SvPosition.xy),
		MaterialParameters.CameraVector,
		SubstrateData,
		Color,
		DualBlendSurfaceCoverage,
		DualBlendSurfaceTransmittancePreCoverage,
		UnlitSurfaceNormal);

#if ((SUBSTRATE_OPAQUE_DEFERRED || (SUBSTRATE_OPAQUE_MATERIAL && SUBSTRATE_FORWARD_SHADING))) && !SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT
	Opacity = 1.0f;

#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
	SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
	SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
	SubstrateMaterialExport = float4(Color, 1.0);
#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
	SubstrateMaterialExport = float4(PixelMaterialInputs.OpacityMask.xxx, 1.0f); // when not masked, OpacityMask default to 1.
#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
	SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
#endif

#else // SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING || SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT
	DualBlendSurfaceLuminancePostCoverage = Color;
	// DualBlendSurfaceTransmittancePreCoverage and DualBlendSurfaceCoverage were fetch during SubstrateUpdateTree
	Color = 0.0f;
	Opacity = 1.0f;

#if SUBSTRATE_USES_CONVERSION_FROM_LEGACY
#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
		SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
		SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
		SubstrateMaterialExport = float4(DualBlendSurfaceLuminancePostCoverage, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
		#if SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODE == SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODULATE
		SubstrateMaterialExport = float4(DualBlendSurfaceLuminancePostCoverage, 1.0f);
		#else
		SubstrateMaterialExport = float4(DualBlendSurfaceLuminancePostCoverage + saturate(1.0 - DualBlendSurfaceCoverage.xxx), 1.0f);
		#endif
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
		SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
	#endif
#else // SUBSTRATE_USES_CONVERSION_FROM_LEGACY
	// The legacy unlit to lightmass conversion is weird and very specialized.
#if	  SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE
		SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_NORMAL
		SubstrateMaterialExport = float4(0.0, 0.0, 0.0, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_EMISSIVE
		SubstrateMaterialExport = float4(DualBlendSurfaceLuminancePostCoverage, 1.0);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE
		SubstrateMaterialExport = float4(DualBlendSurfaceTransmittancePreCoverage * DualBlendSurfaceCoverage + (1.0 - DualBlendSurfaceCoverage), 1.0f);
	#elif SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW
		SubstrateMaterialExport = float4(PixelMaterialInputs.EmissiveColor, 1.0f);
	#endif
#endif // SUBSTRATE_USES_CONVERSION_FROM_LEGACY

#endif
	
#endif

	#if MATERIAL_DOMAIN_POSTPROCESS
		#if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
			Out.MRT[0] = half4(Color, Opacity);
		#else
			Out.MRT[0] = half4(Color, 0);
		#endif
		Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);

	// MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT must come first because it also has MATERIALBLENDING_TRANSLUCENT defined
	#elif MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING

		// Add fog luminance according to surfacecoverage and reduce surface luminance according to fog coverage.
		float3 AdjustedDualBlendAdd = DualBlendSurfaceCoverage * Fogging.rgb + Fogging.a * DualBlendSurfaceLuminancePostCoverage;
		// Fade the surface color transmittance out to 1 according to the surface coverage, and take into account the fog coverage to the surface.
		float3 AdjustedDualBlendMul = lerp(1.0f, Fogging.a * DualBlendSurfaceTransmittancePreCoverage, DualBlendSurfaceCoverage);

		#if DUAL_SOURCE_COLOR_BLENDING_ENABLED
			// no RETURN_COLOR because these values are explicit multiplies and adds
			Out.MRT[0] = half4(AdjustedDualBlendAdd,0.0);
			Out.MRT[1] = half4(AdjustedDualBlendMul,1.0);
		#else
			// In the fallback case, we are blending with the mode 
			float AdjustedAlpha = saturate(1-dot(AdjustedDualBlendMul,float3(1.0f,1.0f,1.0f)/3.0f));
			Out.MRT[0] = half4(AdjustedDualBlendAdd,AdjustedAlpha);
			Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
		#endif

	#elif SUBSTRATE_TRANSLUCENT_MATERIAL

		#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
			DualBlendSurfaceTransmittancePreCoverage= 0.0;
			DualBlendSurfaceCoverage = GetMaterialOpacity(PixelMaterialInputs);
		#endif

			// Additive surfaces should not add any fog color contribution. They should only be attenuated according to fog transmittance.
			// This must be enforced here since coverage can be used to weight the emissive color.
		#if SUBSTRATE_OPTIMIZED_UNLIT && MATERIALBLENDING_ADDITIVE
			Fogging.rgb = 0;
		#endif

			// Add fog luminance according to surfacecoverage and reduce surface luminance according to fog coverage.
			float3 AdjustedDualBlendAdd = DualBlendSurfaceCoverage * Fogging.rgb + Fogging.a * DualBlendSurfaceLuminancePostCoverage;
			// Fade the surface color transmittance out to 1 according to the surface coverage, and take into account the fog coverage to the surface.
			float3 AdjustedDualBlendMul = lerp(1.0f, Fogging.a * DualBlendSurfaceTransmittancePreCoverage, DualBlendSurfaceCoverage);

		#if SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
			// RETURN_COLOR not needed with modulative blending
			half3 FoggedColor = lerp(float3(1, 1, 1), DualBlendSurfaceTransmittancePreCoverage, Fogging.aaa* DualBlendSurfaceCoverage);
			Out.MRT[0] = half4(FoggedColor, 1.0f);
		#elif SUBSTRATE_BLENDING_ALPHAHOLDOUT
			float AdjustedAlpha = saturate(1 - dot(AdjustedDualBlendMul, float3(1.0f, 1.0f, 1.0f) / 3.0f));
			Out.MRT[0] = half4(0.0f.xxx, AdjustedAlpha);
			Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
		#else
			// Pre multipled alpha blending
			float AdjustedAlpha = saturate(1 - dot(AdjustedDualBlendMul, float3(1.0f, 1.0f, 1.0f) / 3.0f));
			#if MATERIALBLENDING_ADDITIVE
				AdjustedAlpha = 0.0f;
			#endif
			// Pre multipled alpha blending
			Out.MRT[0] = half4(AdjustedDualBlendAdd, AdjustedAlpha);
			Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
		#endif

	#elif MATERIALBLENDING_ALPHAHOLDOUT
		// not implemented for holdout
		Out.MRT[0] = half4(Color * Fogging.a + Fogging.rgb * Opacity, Opacity);
		Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
	#elif MATERIALBLENDING_ALPHACOMPOSITE
		Out.MRT[0] = half4(Color * Fogging.a + Fogging.rgb * Opacity, Opacity);
		Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
	#elif MATERIALBLENDING_TRANSLUCENT
		Out.MRT[0] = half4(Color * Fogging.a + Fogging.rgb, Opacity);
		Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
	#elif MATERIALBLENDING_ADDITIVE
		Out.MRT[0] = half4(Color * Fogging.a * Opacity, 0.0f);
		Out.MRT[0] = RETURN_COLOR(Out.MRT[0]);
	#elif MATERIALBLENDING_MODULATE
		// RETURN_COLOR not needed with modulative blending
		half3 FoggedColor = lerp(float3(1, 1, 1), Color, Fogging.aaa * Fogging.aaa);
		Out.MRT[0] = half4(FoggedColor, Opacity);
	#else
		{
			FLightAccumulator LightAccumulator = (FLightAccumulator)0;

			// Apply vertex fog
			Color = Color * Fogging.a + Fogging.rgb;

		#if POST_PROCESS_SUBSURFACE
			// Apply vertex fog to diffuse color
			DiffuseColor = DiffuseColor * Fogging.a + Fogging.rgb;

			if (UseSubsurfaceProfile(GBuffer.ShadingModelID) && 
                View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0 )
			{
				// Adjust for checkerboard. only apply non-diffuse lighting (including emissive) 
				// to the specular component, otherwise lighting is applied twice
				Color *= !bChecker;
			}
			LightAccumulator_Add(LightAccumulator, Color + DiffuseColor, DiffuseColor, 1.0f, UseSubsurfaceProfile(GBuffer.ShadingModelID) || bSubstrateSubsurfaceEnable);
		#else
			LightAccumulator_Add(LightAccumulator, Color, 0, 1.0f, false);
		#endif
			Out.MRT[0] = RETURN_COLOR(LightAccumulator_GetResult(LightAccumulator));

			#if !USES_GBUFFER
				// Without deferred shading the SSS pass will not be run to reset scene color alpha for opaque / masked to 0
				// Scene color alpha is used by scene captures and planar reflections
				Out.MRT[0].a = 0;
			#endif
		}
	#endif

	#if USES_GBUFFER
		// -0.5 .. 0.5, could be optimzed as lower quality noise would be sufficient
		float QuantizationBias = PseudoRandom( MaterialParameters.SvPosition.xy ) - 0.5f;

		GBuffer.IndirectIrradiance = IndirectIrradiance;
		GBuffer.NormalDistribution = 0.0f;

		// this is the new encode, the older encode is the #else, keeping it around briefly until the new version is confirmed stable.
		#if 1
		{
			// change this so that we can pack everything into the gbuffer, but leave this for now
			#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
				GBuffer.GenericAO = float(GBuffer.DiffuseIndirectSampleOcclusion) * (1.0f / 255.0f);
			#elif ALLOW_STATIC_LIGHTING
				// No space for AO. Multiply IndirectIrradiance by AO instead of storing.
				GBuffer.GenericAO = EncodeIndirectIrradiance(GBuffer.IndirectIrradiance * GBuffer.GBufferAO) + QuantizationBias * (1.0 / 255.0);	// Stationary sky light path
			#else
				GBuffer.GenericAO = GBuffer.GBufferAO;	// Movable sky light path
			#endif

			EncodeGBufferToMRT(Out, GBuffer, QuantizationBias);

			if (GBuffer.ShadingModelID == SHADINGMODELID_UNLIT && (!SUBSTRATE_ENABLED || SUBSTRATE_GBUFFER_FORMAT==0)) // Do not touch what Substrate outputs with Format=1
			{
				Out.MRT[1] = 0;
				SetGBufferForUnlit(Out.MRT[2]);
				Out.MRT[3] = 0;
				Out.MRT[GBUFFER_HAS_VELOCITY ? 5 : 4] = 0;
				Out.MRT[GBUFFER_HAS_VELOCITY ? 6 : 5] = 0;
			}

		#if SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT
			// In deferred, we always output the directional light in a separated buffer.
			// This is used to apply distance field shadows or light function to the main directional light.
			// Substrate also writes it through MRT because this is faster than through UAV.
			#if SUBSTRATE_ENABLED && SUBSTRATE_INLINE_SINGLELAYERWATER
				#if SUBSTRATE_GBUFFER_FORMAT==1
				Out.MRT[(GBUFFER_HAS_VELOCITY ? 2 : 1) + (GBUFFER_HAS_PRECSHADOWFACTOR ? 1 : 0)] = float4(SeparatedWaterMainDirLightLuminance * View.PreExposure, 1.0f);
				#else
				Out.MRT[(GBUFFER_HAS_VELOCITY ? 5 : 4)] = float4(SeparatedWaterMainDirLightLuminance * View.PreExposure, 1.0f);
				#endif
			#else
			if (GBuffer.ShadingModelID == SHADINGMODELID_SINGLELAYERWATER)
			{
				Out.MRT[(GBUFFER_HAS_VELOCITY ? 5 : 4)] = float4(SeparatedWaterMainDirLightLuminance * View.PreExposure, 1.0f);
			}
			#endif
		#endif

			Out.SGGX = 0.0f;
		#if NANITE_USE_ANALYTIC_SGGX && IS_NANITE_PASS && MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
			if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
			{
				GBuffer.NormalDistribution = MaterialParameters.NormalDistribution;
				Out.SGGX = float4(EncodeNormal(GBuffer.NormalDistribution), 1.0f);
			}
		#endif
		}
		#else
		{
			float4 OutGBufferA = 0;
			float4 OutGBufferB = 0;
			float4 OutGBufferC = 0;

			EncodeGBuffer(GBuffer, OutGBufferA, OutGBufferB, OutGBufferC, OutGBufferD, OutGBufferE, OutVelocity, QuantizationBias);

			Out.MRT[1] = OutGBufferA;
			Out.MRT[2] = OutGBufferB;
			Out.MRT[3] = OutGBufferC;

			#if GBUFFER_HAS_VELOCITY
				Out.MRT[4] = OutVelocity;
			#endif

			Out.MRT[GBUFFER_HAS_VELOCITY ? 5 : 4] = OutGBufferD;

			#if GBUFFER_HAS_PRECSHADOWFACTOR
				Out.MRT[GBUFFER_HAS_VELOCITY ? 6 : 5] = OutGBufferE;
			#endif
		}
		#endif

	#else

		// If not using the full gbuffer (forward shading) the velocity buffer can still be written to in the basepass.
		#if GBUFFER_HAS_VELOCITY && !DUAL_SOURCE_COLOR_BLENDING_ENABLED
			Out.MRT[1] = OutVelocity;
		#endif

	#endif 

	if(bEditorWeightedZBuffering)
	{
		Out.MRT[0].a = 1;

		#if MATERIALBLENDING_MASKED
			// some material might have a opacity value
			Out.MRT[0].a = GetMaterialMaskInputRaw(PixelMaterialInputs);
		#endif

		#if EDITOR_ALPHA2COVERAGE != 0
			// per MSAA sample
			if(View.NumSceneColorMSAASamples > 1)
			{
				Out.Coverage = In.Coverage & CustomAlpha2Coverage(Out.MRT[0]);
			}
			else
			{
				// no MSAA is handle like per pixel
				clip(Out.MRT[0].a - GetMaterialOpacityMaskClipValue());
			}
		#else
			// per pixel
			clip(Out.MRT[0].a - GetMaterialOpacityMaskClipValue());
		#endif
	}

#if !MATERIALBLENDING_MODULATE && !SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
#if MATERIAL_IS_SKY
	// Dynamic capture exposure is 1 as of today.
	const float ViewPreExposure = View.RealTimeReflectionCapture>0.0f ? View.RealTimeReflectionCapturePreExposure : View.PreExposure;
#else
	const float ViewPreExposure = View.PreExposure;
#endif
	// We need to multiply pre-exposure by all components including A, otherwise the ratio of
	// diffuse to specular lighting will get messed up in the SSS pass.
	// RGB: Full color (Diffuse + Specular)
	// A:   Diffuse Intensity, but only if we are not blending
	#if MATERIAL_DOMAIN_POSTPROCESS || MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT || MATERIALBLENDING_ALPHAHOLDOUT || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE
		Out.MRT[0].rgb  *= ViewPreExposure;
	#else
		Out.MRT[0].rgba *= ViewPreExposure;
	#endif
#endif

	// If support OIT, then remove blending contribution and insert OIT sample
	// Note: Out.MRT[0] has already view pre-exposition applied
#if OIT_ENABLED
	if (TranslucentBasePass.OIT.bOITEnable)
	{
		const float OpacityThreshold = 10e-4f;

		#if MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING
		{
			#if !SUBSTRATE_ENABLED
			float3 AdjustedDualBlendAdd = Out.MRT[0];
			float3 AdjustedDualBlendMul = Out.MRT[1];
			#endif

			// Add early out threshold?
			AddOITSample(uint2(In.SvPosition.xy), AdjustedDualBlendAdd, AdjustedDualBlendMul, MaterialParameters.ScreenPosition.w);
			Out.MRT[0] = half4(0, 0, 0, 0);
			Out.MRT[1] = half4(1, 1, 1, 1);
		}
		#elif SUBSTRATE_TRANSLUCENT_MATERIAL && !MATERIAL_DOMAIN_POSTPROCESS
		{
			#if SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
			AddOITSample(uint2(In.SvPosition.xy), float3(0, 0, 0), FoggedColor, MaterialParameters.ScreenPosition.w);
			Out.MRT[0] = half4(1, 1, 1, 1);
			Out.MRT[1] = half4(0, 0, 0, 0);
			#else
			AddOITSample(uint2(In.SvPosition.xy), AdjustedDualBlendAdd, AdjustedDualBlendMul, MaterialParameters.ScreenPosition.w);
			Out.MRT[0] = half4(0, 0, 0, 0);
			Out.MRT[1] = half4(1, 1, 1, 1);
			#endif
		}
		#else
		{
			float3 WriteColor = Out.MRT[0].rgb;
		#if MATERIALBLENDING_ADDITIVE
			// NOP
		#elif MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_ALPHAHOLDOUT
			// Not clear what should be done in this case.
		#elif MATERIALBLENDING_TRANSLUCENT
			WriteColor *= Opacity; // WriteColor is premultiplied alpha color
		#else
			#error OIT is not support for this blend mode
		#endif

			if (Opacity > OpacityThreshold)
			{
				AddOITSample(uint2(In.SvPosition.xy), WriteColor, 1.f - Opacity, MaterialParameters.ScreenPosition.w);
			}
			Out.MRT[0] = half4(0, 0, 0, 0);
		}
		#endif
	}
#endif // OIT_ENABLED

	Out.MRT[0].xyz = min(Out.MRT[0].xyz, View.MaterialMaxEmissiveValue.xxx);

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED
#if OUTPUT_PIXEL_DEPTH_OFFSET
	// When in deferred, opaque materials with pixel depth offset must execute a custom depth test in order to avoid conflicting UAV writes.
	if (ManualDepthTestEqual(In.SvPosition, Out.Depth))
#endif
	{
		FinalizeVirtualTextureFeedback(
			MaterialParameters.VirtualTextureFeedback,
			MaterialParameters.SvPosition,
			View.VTFeedbackBuffer,
			VISUALIZE
		);
	}
#endif

#if SUBSTRATE_MATERIAL_EXPORT_EXECUTED
	Out.MRT[0] = SubstrateMaterialExport;
#endif

#if SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT
		
	//Calculate IoR for translucent material with ior and update holdout property
	#if (MATERIALBLENDING_TRANSLUCENT || SUBSTRATE_TRANSLUCENT_MATERIAL) && (REFRACTION_USE_INDEX_OF_REFRACTION || REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0)
		#if REFRACTION_USE_INDEX_OF_REFRACTION
			FMaterialRefractionData RefractionData = GetMaterialRefraction(PixelMaterialInputs);
			float MaterialIOR = GetMaterialRefractionIOR(RefractionData);
		#else // REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0
			float MaterialIOR = DielectricF0RGBToIor(ComputeF0(Specular, BaseColor, Metallic));
		#endif
		
		// When material IOR is active and not 1.0f, it is fully opaque and match path tracing.
		#if SUBSTRATE_TRANSLUCENT_MATERIAL
			#if !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && !SUBSTRATE_BLENDING_ALPHAHOLDOUT
				float DualBlendSurfaceCoverageMix = 1.0f;
				DualBlendSurfaceCoverage = select(MaterialIOR != 0.0 && MaterialIOR != 1.0f, DualBlendSurfaceCoverageMix, DualBlendSurfaceCoverage);
			#endif
		#elif MATERIALBLENDING_TRANSLUCENT
			float OpacityMix = 1.0f;
			Opacity = select(MaterialIOR != 0.0 && MaterialIOR != 1.0f, OpacityMix, Opacity);
		#endif
	#endif 
	
	// Only include the holdout logic if the Renderer is forwarding its alpha channel to post-processing
	if (bIsHoldout)
	{
		// Discard this pixel so that nothing gets written to the G-Buffers and alpha is kept as its default value (background)
#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
		// This catches the common case as well as handles cases where Opacity has a different meaning, such as for Subsurface materials
		Out.MRT[0] = half4(0.0, 0.0, 0.0, 1.0);
#elif MATERIALBLENDING_MODULATE
		Out.MRT[0] = half4(0.0, 0.0, 0.0, Opacity);
		// Alternative method. Keep MRT[0] as it is to get a closer look to path tracing:
		//	Out.MRT[0] = half4(FoggedColor, Opacity);
		// MATERIALBLENDING_MODULATE comments in the holdout pass.
#elif SUBSTRATE_TRANSLUCENT_MATERIAL
	#if SUBSTRATE_BLENDING_ALPHAHOLDOUT
		//This is not the translucent holdout pass, do not reset Out.MRT[0] so holdout can accumulate color contribution.
	#elif SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE
		Out.MRT[0] = half4(0.0, 0.0, 0.0, DualBlendSurfaceCoverage);
	#elif SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE
		Out.MRT[0] = half4(0.0, 0.0, 0.0, DualBlendSurfaceCoverage);
	#endif
			
#else
	#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
		Out.MRT[0] = half4(0.0, 0.0, 0.0, DualBlendSurfaceCoverage);
	#else
		// TODO: Not all translucent cases get the right alpha (compared to the path traced reference)
		Out.MRT[0] = half4(0.0, 0.0, 0.0, Opacity);
	#endif
#endif

#if USES_GBUFFER
		Out.MRT[1] = 0.0;
		Out.MRT[2] = 0.0;
		Out.MRT[3] = 0.0;
#endif
	}

#if MATERIALBLENDING_ANY_TRANSLUCENT || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_ALPHAHOLDOUT
	// If this is translucency holdout pass, calculate the background visibility and path throughput.
	if (bIsTranslucentHoldoutPass && ResolvedView.bPrimitiveAlphaHoldoutEnabled)
	{
		// Default behavior is to preserve the destination alpha.
		half BackgroundVisibilityAdd = 0.0f;
		half PathThroughputMul = 1.0f;		
		half FogPathThroughput = Fogging.a;
		half PrimitiveCoverageForFog = 0.0f;
	
#if SUBSTRATE_TRANSLUCENT_MATERIAL
	#if SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE
		BackgroundVisibilityAdd = bIsHoldout ? DualBlendSurfaceCoverage : 0;
		PathThroughputMul = 1 - DualBlendSurfaceCoverage;
		PrimitiveCoverageForFog = DualBlendSurfaceCoverage;
	#elif SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE
		BackgroundVisibilityAdd = bIsHoldout ? DualBlendSurfaceCoverage : 0;
		PathThroughputMul = 1 - DualBlendSurfaceCoverage;
		PrimitiveCoverageForFog = DualBlendSurfaceCoverage;
	#elif SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
		// TODO
	#elif MATERIALBLENDING_ALPHAHOLDOUT
		//alpha holdout should always contribute to the background visibility.
		BackgroundVisibilityAdd = 1 - AdjustedAlpha;
		PathThroughputMul = AdjustedAlpha;
		PrimitiveCoverageForFog = 1 - AdjustedAlpha;
	#endif 
#else

	#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
		//thin translucent shading model uses surface coverage as opacity.
		BackgroundVisibilityAdd = bIsHoldout ? DualBlendSurfaceCoverage : 0;
		PathThroughputMul = 1 - DualBlendSurfaceCoverage;
		PrimitiveCoverageForFog = DualBlendSurfaceCoverage;
	#elif MATERIALBLENDING_ALPHAHOLDOUT
		//alpha holdout should always contribute to the background visibility.
		BackgroundVisibilityAdd = Opacity;
		PathThroughputMul = (1 - Opacity);
		PrimitiveCoverageForFog = Opacity;
	#elif MATERIALBLENDING_TRANSLUCENT
		// Use Opacity as it is.
		BackgroundVisibilityAdd = bIsHoldout ? Opacity : 0;
		PathThroughputMul = 1 - Opacity;
		PrimitiveCoverageForFog = Opacity;
	#endif
#endif

#if MATERIALBLENDING_ADDITIVE
		// Not affecting alpha
		BackgroundVisibilityAdd = 0.0f;
		PathThroughputMul = 1.0f;
		PrimitiveCoverageForFog = 0.0f;
#elif MATERIALBLENDING_MODULATE
		FogPathThroughput = Fogging.a * Fogging.a; // Match rendering pass
		BackgroundVisibilityAdd = bIsHoldout ? Opacity : 0;
		PathThroughputMul = (1 - Opacity);
		PrimitiveCoverageForFog = Opacity;
		//---------------------------------------------------------------------
		// Another solution: use fogged color to calcualte throughput and background visibility
		// C = C * L + C * (1-L).
		// BackgroundVisibilityAdd = bIsHoldout ? (1-Luminance(FoggedColor)) : 0;
		// PathThroughputMul = Luminance(FoggedColor);
		// Issue: Need to fix modulate layer stacking does not match patch tracing.
		// Otherwise, we can only have one modulate in a scene that can be holdout as well.
#endif

		// Fog holdout parameters
		const bool bIsExponentialFogHoldout = IsExponentialFogHoldout(View.EnvironmentComponentsFlags); 
		half FogBackgroundVisibilityAdd = bIsExponentialFogHoldout ? (PrimitiveCoverageForFog * (1 - FogPathThroughput)) : 0;
	
		// Render far to close logic
		// Holdout = Holdout * PrimitivePathThroughput + (FogHoldout * PrimCoverage + HoldoutPrimCoverage * FogPathThroughput);
		Out.MRT[0].xyz = FogBackgroundVisibilityAdd + BackgroundVisibilityAdd * FogPathThroughput;
		Out.MRT[0].w = PathThroughputMul;
		
	}
#endif
#endif // SUPPORT_MATERIAL_PRIMITIVE_ALPHA_HOLDOUT

}

#if SUBSTRATE_OPAQUE_DEFERRED

// Substrate opaque deferred materials require early depth stencil test to avoid writing through UAV from the pixel shader when the depth surface is not from the current pixel shader material
// Notes: 
//		- Pixel depth offset prevents the usage of early depth test because the depth is generated in shader. So we can only relay on LateZ in this case.
//		- DEPTHSTENCIL_EARLYTEST_LATEWRITE does not help in this case due to UAV writes we need to skip.
//		- In order to avoid UAV writes, we will run in shader manual depth tests.
//		- We also disable early depth test when runnning editor primitive alpha-to-coverage (for instance masked LevelGridMaterial)
#if IS_NANITE_PASS
	// Even with masking and/or pixel depth offset, Nanite never uses actual clip() or SV_Depth
	#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
#elif COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE
	// If we support early depth test with late write behaviour then use it since we may be using discard, or modifying depth
	#define PIXELSHADER_EARLYDEPTHSTENCIL DEPTHSTENCIL_EARLYTEST_LATEWRITE
#elif !OUTPUT_PIXEL_DEPTH_OFFSET && !(EDITOR_ALPHA2COVERAGE && MATERIALBLENDING_MASKED)
	#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
#endif

#else // SUBSTRATE_OPAQUE_DEFERRED

// If virtual texture feedback is enabled then use early depth test so that UAV feedback buffer writes respect the depth test
#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED || OIT_ENABLED
	#if IS_NANITE_PASS
		// Even with masking and/or pixel depth offset, Nanite never uses actual clip() or SV_Depth
		#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
	#elif COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE
		// If we support early depth test with late write behaviour then use it since we may be using discard, or modifying depth
		#define PIXELSHADER_EARLYDEPTHSTENCIL DEPTHSTENCIL_EARLYTEST_LATEWRITE
	#elif !OUTPUT_PIXEL_DEPTH_OFFSET
		// Otherwise we can only use early depth test if not modifying depth
		// Modifying depth will trigger the slow path where we write feedback to UAV even where depth occluded!
		#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
	#endif
#endif

#endif // SUBSTRATE_OPAQUE_DEFERRED

// Force early Z on single layer water main pass.
// This assumes that a full depth prepass was run for SLW and that the depth test is set to equal in the main pass.
#ifndef PIXELSHADER_EARLYDEPTHSTENCIL
	#if SINGLE_LAYER_WATER_NO_DISCARD && !OUTPUT_PIXEL_DEPTH_OFFSET
		#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
	#endif // SINGLE_LAYER_WATER_NO_DISCARD && !OUTPUT_PIXEL_DEPTH_OFFSET
#endif // PIXELSHADER_EARLYDEPTHSTENCIL

// the following needs to match to the code in FSceneTextures::GetGBufferRenderTargets()
#define PIXELSHADEROUTPUT_BASEPASS 1
//#if USES_GBUFFER
//#define PIXELSHADEROUTPUT_MRT0 (!SELECTIVE_BASEPASS_OUTPUTS || NEEDS_BASEPASS_VERTEX_FOGGING || USES_EMISSIVE_COLOR || ALLOW_STATIC_LIGHTING || MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
//#define PIXELSHADEROUTPUT_MRT1 ((!SELECTIVE_BASEPASS_OUTPUTS || !MATERIAL_SHADINGMODEL_UNLIT))
//#define PIXELSHADEROUTPUT_MRT2 ((!SELECTIVE_BASEPASS_OUTPUTS || !MATERIAL_SHADINGMODEL_UNLIT))
//#define PIXELSHADEROUTPUT_MRT3 ((!SELECTIVE_BASEPASS_OUTPUTS || !MATERIAL_SHADINGMODEL_UNLIT))
//	#if GBUFFER_HAS_VELOCITY
//		#define PIXELSHADEROUTPUT_MRT4 WRITES_VELOCITY_TO_GBUFFER
//		#define PIXELSHADEROUTPUT_MRT5 (!SELECTIVE_BASEPASS_OUTPUTS || WRITES_CUSTOMDATA_TO_GBUFFER)
//		#define PIXELSHADEROUTPUT_MRT6 (GBUFFER_HAS_PRECSHADOWFACTOR && (!SELECTIVE_BASEPASS_OUTPUTS || WRITES_PRECSHADOWFACTOR_TO_GBUFFER && !MATERIAL_SHADINGMODEL_UNLIT))
//	#else //GBUFFER_HAS_VELOCITY
//		#define PIXELSHADEROUTPUT_MRT4 (!SELECTIVE_BASEPASS_OUTPUTS || WRITES_CUSTOMDATA_TO_GBUFFER)
//		#define PIXELSHADEROUTPUT_MRT5 (GBUFFER_HAS_PRECSHADOWFACTOR && (!SELECTIVE_BASEPASS_OUTPUTS || WRITES_PRECSHADOWFACTOR_TO_GBUFFER && !MATERIAL_SHADINGMODEL_UNLIT))
//	#endif //GBUFFER_HAS_VELOCITY
//#else //USES_GBUFFER
//	#define PIXELSHADEROUTPUT_MRT0 1
//	// we also need MRT for thin translucency due to dual blending if we are not on the fallback path
//	#define PIXELSHADEROUTPUT_MRT1 (WRITES_VELOCITY_TO_GBUFFER || (MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && THIN_TRANSLUCENT_USE_DUAL_BLEND))
//#endif //USES_GBUFFER
#define PIXELSHADEROUTPUT_A2C ((EDITOR_ALPHA2COVERAGE) != 0)
#define PIXELSHADEROUTPUT_COVERAGE ((OUTPUTS_COVERAGE) != 0)

// all PIXELSHADEROUTPUT_ and "void FPixelShaderInOut_MainPS()" need to be setup before this include
// this include generates the wrapper code to call MainPS(inout FPixelShaderOutput PixelShaderOutput)
#if COMPUTE_SHADED
#include "ComputeShaderOutputCommon.ush"
#else
#include "PixelShaderOutputCommon.ush"
#endif
