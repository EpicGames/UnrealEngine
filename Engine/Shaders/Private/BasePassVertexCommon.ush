// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassVertexCommon.usf: Vertex-related base pass definitions, used by vertex,
	hull and domain shader
=============================================================================*/

#include "Common.ush"

// Reroute SceneTexturesStruct uniform buffer references to the appropriate base pass uniform buffer
#if MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE
	#define SceneTexturesStruct TranslucentBasePass.SceneTextures
#endif 

#include "/Engine/Generated/Material.ush"
#include "BasePassCommon.ush"
#include "/Engine/Generated/VertexFactory.ush"

#if NEEDS_BASEPASS_VERTEX_FOGGING
	#include "HeightFogCommon.ush"
#endif

struct FBasePassVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	FBasePassInterpolantsVSToPS BasePassInterpolants;
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
	FStereoVSOutput StereoOutput;
#if USE_GLOBAL_CLIP_PLANE && MESHSHADER
	float GlobalClipPlaneDistance : SV_CLIPDISTANCE;
#endif
};

#define FBasePassVSOutput FBasePassVSToPS
#define VertexFactoryGetInterpolants VertexFactoryGetInterpolantsVSToPS
