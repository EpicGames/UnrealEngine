// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassVertexShader.usf: Base pass vertex shader
=============================================================================*/

#include "BasePassVertexCommon.ush"
#include "SHCommon.ush"
#include "VolumetricLightmapShared.ush"

#if TRANSLUCENCY_PERVERTEX_FORWARD_SHADING
#include "ReflectionEnvironmentShared.ush"

// Enables applying a cloud shadow factor to lighting calculations in ForwardLightingCommon.ush.
#define ENABLE_FORWARD_CLOUD_SHADOW SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT
#include "ForwardLightingCommon.ush"
#endif

#if NEEDS_BASEPASS_VERTEX_FOGGING && PROJECT_SUPPORT_SKY_ATMOSPHERE
#include "SkyAtmosphereCommon.ush"
#endif

#if (NEEDS_BASEPASS_VERTEX_FOGGING && MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING) || SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT || NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
#include "VolumetricCloudCommon.ush"
#endif

#if LOCAL_FOG_VOLUME_ON_TRANSLUCENT
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

/** Entry point for the base pass vertex shader. */
#if MESHSHADER
void Main_Inner(
#else
void Main(
#endif
	FVertexFactoryInput Input,
	out FBasePassVSOutput Output
#if USE_GLOBAL_CLIP_PLANE && !MESHSHADER
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, Output.StereoOutput);
	const uint EyeIndex = GetEyeIndexFromVF(Input);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPositionExcludingWPO = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 WorldPosition = WorldPositionExcludingWPO;
	float4 ClipSpacePosition;

	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);

	// Isolate instructions used for world position offset
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	{
		WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
	}

	{
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
		ClipSpacePosition = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);
#if MATERIAL_IS_SKY
		if (IsOrthoProjection())
		{
			// Ortho rendering doesn't stretch to LWC typically, so clamp the sky to as close to the reverse-z far plane as possible unless the dome is already closer to the camera.
			if (ConvertFromDeviceZ(ClipSpacePosition.z) > ResolvedView.OrthoFarPlane)
			{
				ClipSpacePosition.z = FarDepthValue;
			}
		}
#endif
		Output.Position = INVARIANT(ClipSpacePosition);
	}

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif
	#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
		Output.BasePassInterpolants.PixelPositionExcludingWPO = WorldPositionExcludingWPO.xyz;
	#endif

	Output.FactoryInterpolants = VertexFactoryGetInterpolants(Input, VFIntermediates, VertexParameters);

#if STORE_CLIP_ZW_IN_INTERPOLATOR
	Output.FactoryInterpolants.ClipZW = Output.Position.zw;
#endif

// Calculate the fog needed for translucency
#if NEEDS_BASEPASS_VERTEX_FOGGING
	Output.BasePassInterpolants.VertexFog = CalculateHeightFog(WorldPosition.xyz - ResolvedView.TranslatedWorldCameraOrigin, EyeIndex, ResolvedView); // WorldPosition is in fact TranslatedWorldPosition 

	const float OneOverPreExposure = ResolvedView.OneOverPreExposure;

#if PROJECT_SUPPORT_SKY_ATMOSPHERE && BASEPASS_SKYATMOSPHERE_AERIALPERSPECTIVE && MATERIAL_IS_SKY==0 // Do not apply aerial perpsective on sky materials
	if (ResolvedView.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f)
	{
		// Sample the aerial perspective (AP). It is also blended under the VertexFog parameter.
		Output.BasePassInterpolants.VertexFog = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			Output.Position, (WorldPosition.xyz - ResolvedView.TranslatedWorldCameraOrigin) * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			OneOverPreExposure, Output.BasePassInterpolants.VertexFog);
	}
#endif
#endif // NEEDS_BASEPASS_VERTEX_FOGGING

#if NEEDS_BASEPASS_VERTEX_FOGGING

	// Apply local fog volume to if enabled. It is always enabled per vertex as of today, that is why it is out ofr the NEEDS_BASEPASS_VERTEX_FOGGING defines.
#if LOCAL_FOG_VOLUME_ON_TRANSLUCENT
	float4 VertexClipSpacePosition			= mul(float4(WorldPosition.xyz, 1), ResolvedView.TranslatedWorldToClip);
	float2 SvPosition						= (VertexClipSpacePosition.xy / VertexClipSpacePosition.w * float2(.5f, -.5f) + .5f) * ResolvedView.ViewSizeAndInvSize.xy;
	uint2 TilePos							= clamp(uint2(SvPosition.xy / float(LFVTilePixelSize)), uint2(0, 0), LFVTileDataResolution - 1);
	float4 LFVContribution					= GetLFVContribution(ResolvedView, TilePos, WorldPosition.xyz);
	Output.BasePassInterpolants.VertexFog	= float4(LFVContribution.rgb + Output.BasePassInterpolants.VertexFog.rgb * LFVContribution.a, LFVContribution.a * Output.BasePassInterpolants.VertexFog.a);
#endif // LOCAL_FOG_VOLUME_ON_TRANSLUCENT

#if MATERIAL_ENABLE_TRANSLUCENCY_CLOUD_FOGGING
	if (TranslucentBasePass.ApplyVolumetricCloudOnTransparent > 0.0f)
	{
		Output.BasePassInterpolants.VertexFog = GetCloudLuminanceTransmittanceOverFog(
			Output.Position, WorldPosition.xyz, ResolvedView.TranslatedWorldCameraOrigin,
			TranslucentBasePass.VolumetricCloudColor, TranslucentBasePass.VolumetricCloudColorSampler,
			TranslucentBasePass.VolumetricCloudDepth, TranslucentBasePass.VolumetricCloudDepthSampler,
			OneOverPreExposure, Output.BasePassInterpolants.VertexFog, TranslucentBasePass.SoftBlendingDistanceKm, 
			TranslucentBasePass.VolumetricCloudColorUVScale, TranslucentBasePass.VolumetricCloudColorUVMax);
	}
#endif

#endif // NEEDS_BASEPASS_VERTEX_FOGGING

#if TRANSLUCENCY_ANY_PERVERTEX_LIGHTING || SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT || NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
	float3 WorldPositionForVertexLightingTranslated = VertexFactoryGetPositionForVertexLighting(Input, VFIntermediates, WorldPosition.xyz);
	float3 WorldPositionForVertexLighting = WorldPositionForVertexLightingTranslated - DFHackToFloat(ResolvedView.PreViewTranslation);
#endif

#if TRANSLUCENCY_PERVERTEX_LIGHTING_VOLUME

	float3 InnerVolumeUVs;
	float3 OuterVolumeUVs;
	float FinalLerpFactor;

	//@todo - get from VF
	float3 LightingPositionOffset = 0;
	ComputeVolumeUVs(WorldPositionForVertexLightingTranslated, LightingPositionOffset, InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

	#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
	
		Output.BasePassInterpolants.AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor).xyz;
		Output.BasePassInterpolants.DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

	#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

		Output.BasePassInterpolants.AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor).xyz;

	#endif

#elif TRANSLUCENCY_PERVERTEX_FORWARD_SHADING

	float3 InnerVolumeUVs;
	float3 OuterVolumeUVs;
	float FinalLerpFactor;

	//@todo - get from VF
	float3 LightingPositionOffset = 0;
	ComputeVolumeUVs(WorldPositionForVertexLightingTranslated, LightingPositionOffset, InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

	if (TranslucencyLightingForwardUsingVolume)
	{
		float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		float3 DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
		FTwoBandSHVectorRGB TranslucentLighting = ReconstructSHCoefficients(AmbientLightingVector, DirectionalLightingVector);

		Output.BasePassInterpolants.VertexDiffuseLighting = GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, VertexParameters.TangentToWorld[2], GetMaterialTranslucencyDirectionalLightingIntensity()).rgb;
	}
	else
	{
		float4 VertexLightingClipSpacePosition = mul(float4(WorldPositionForVertexLightingTranslated, 1), ResolvedView.TranslatedWorldToClip);
		float2 SvPosition = (VertexLightingClipSpacePosition.xy / VertexLightingClipSpacePosition.w * float2(.5f, -.5f) + .5f) * ResolvedView.ViewSizeAndInvSize.xy;
		uint GridIndex = ComputeLightGridCellIndex((uint2)(SvPosition* View.LightProbeSizeRatioAndInvSizeRatio.zw - ResolvedView.ViewRectMin.xy), VertexLightingClipSpacePosition.w, EyeIndex);

		float DirectionalLightCloudShadow = 1.0f;
		#if SUPPORT_CLOUD_SHADOW_ON_FORWARD_LIT_TRANSLUCENT
			float OutOpticalDepth = 0.0f;
			if (TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapStrength > 0.0f)
			{
				DirectionalLightCloudShadow = lerp(
					1.0f,
					GetCloudVolumetricShadow(
						WorldPositionForVertexLightingTranslated,
						TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapTranslatedWorldToLightClipMatrix,
						TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapFarDepthKm,
						TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapTexture,
						TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapSampler,
						OutOpticalDepth),
					TranslucentBasePass.ForwardDirLightCloudShadow.CloudShadowmapStrength);
			}
		#endif

		Output.BasePassInterpolants.VertexDiffuseLighting = GetForwardDirectLightingForVertexLighting(GridIndex, WorldPositionForVertexLightingTranslated, Output.Position.w, VertexParameters.TangentToWorld[2], EyeIndex, DirectionalLightCloudShadow);
	}

#elif NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#define ForwardDirLightCloudShadowParams SingleLayerWater.ForwardDirLightCloudShadow
#elif IS_MATERIAL_TRANSLUCENT_AND_LIT
#define ForwardDirLightCloudShadowParams TranslucentBasePass.ForwardDirLightCloudShadow
#endif

	Output.BasePassInterpolants.VertexCloudShadow = 1.0f;
	if (ForwardDirLightCloudShadowParams.CloudShadowmapStrength > 0.0f)
	{
		float OutOpticalDepth = 0.0f;
		Output.BasePassInterpolants.VertexCloudShadow = lerp(
			1.0f,
			GetCloudVolumetricShadow(
				WorldPositionForVertexLightingTranslated,
				ForwardDirLightCloudShadowParams.CloudShadowmapTranslatedWorldToLightClipMatrix,
				ForwardDirLightCloudShadowParams.CloudShadowmapFarDepthKm,
				ForwardDirLightCloudShadowParams.CloudShadowmapTexture,
				ForwardDirLightCloudShadowParams.CloudShadowmapSampler,
				OutOpticalDepth),
			ForwardDirLightCloudShadowParams.CloudShadowmapStrength);
	}

#undef ForwardDirLightCloudShadowParams

#endif

	#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING && TRANSLUCENCY_ANY_PERVERTEX_LIGHTING
		float3 BrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WorldPositionForVertexLighting);

		#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL
			FOneBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH1(BrickTextureUVs);
			Output.BasePassInterpolants.VertexIndirectAmbient = float3(IrradianceSH.R.V, IrradianceSH.G.V, IrradianceSH.B.V);
		#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
			// Need to interpolate directional lighting so we can incorporate a normal in the pixel shader
			FTwoBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH2(BrickTextureUVs);
			Output.BasePassInterpolants.VertexIndirectSH[0] = IrradianceSH.R.V;
			Output.BasePassInterpolants.VertexIndirectSH[1] = IrradianceSH.G.V;
			Output.BasePassInterpolants.VertexIndirectSH[2] = IrradianceSH.B.V;
		#endif
	#endif

#if WRITES_VELOCITY_TO_GBUFFER
	{
		float4 PrevTranslatedWorldPosition = float4(0, 0, 0, 1);
		// Use 'FLATTEN' instead of 'BRANCH' because of nVidia driver compiler bug: see UE-108339 - nVidia is working on a driver side fix right now
		FLATTEN
		if ((GetPrimitiveData(VertexParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_OUTPUT_VELOCITY) != 0)
		{
			PrevTranslatedWorldPosition = VertexFactoryGetPreviousWorldPosition( Input, VFIntermediates );	
			VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, PrevTranslatedWorldPosition.xyz, TangentToLocal, true);
			PrevTranslatedWorldPosition.xyz += GetMaterialPreviousWorldPositionOffset(VertexParameters);
			ApplyMaterialPreviousFirstPersonTransform(VertexParameters, PrevTranslatedWorldPosition.xyz);
	
			PrevTranslatedWorldPosition = mul(float4(PrevTranslatedWorldPosition.xyz, 1), ResolvedView.PrevTranslatedWorldToClip);
		}

		// compute the old screen pos with the old world position and the old camera matrix
		Output.BasePassInterpolants.VelocityPrevScreenPosition = INVARIANT(PrevTranslatedWorldPosition); 
	}
#endif	// WRITES_VELOCITY_TO_GBUFFER
}

#if MESHSHADER
MESH_SHADER_TRIANGLE_ATTRIBUTES(BASEPASS_MESH_SHADER_TG_SIZE)
void Main(
	uint GroupThreadID : SV_GroupThreadID,
	uint3 GroupID : SV_GroupID,
	MESH_SHADER_VERTEX_EXPORT(FBasePassVSOutput, 32),
	MESH_SHADER_TRIANGLE_EXPORT(32)
	)
{
	uint LaneVertIndex = 0;

	FVertexFactoryMeshShaderIntermediates MSIntermediates;
	MSIntermediates = VertexFactoryMeshShaderGetIntermediates(GroupThreadID, GroupID);

	VertexFactoryDedupMeshShaderVertIndexes(MSIntermediates, GroupThreadID, LaneVertIndex);

	uint UniqueVertexCount;
	uint TriangleCount;
	VertexFactoryGetMeshShaderOutputCount(MSIntermediates, UniqueVertexCount, TriangleCount);

	SetMeshOutputCounts(UniqueVertexCount, TriangleCount);

	BRANCH
	if (VertexFactoryMeshShaderIsValidIndex(MSIntermediates))
	{
		uint PrimExportIndex = GroupThreadID;
		if (VertexFactoryMeshShaderIsPrimitiveValid(MSIntermediates, PrimExportIndex))
		{
			MESH_SHADER_WRITE_TRIANGLE(PrimExportIndex, MSIntermediates.VertIndexes);
		}

		uint VertExportIndex = GroupThreadID;
		if (VertexFactoryMeshShaderIsVertexValid(MSIntermediates, VertExportIndex))
		{
			FVertexFactoryInput VertexInput = VertexFactoryMeshShaderGetInput(MSIntermediates, LaneVertIndex);
			FBasePassVSOutput VertexOutput = (FBasePassVSOutput)0;
			Main_Inner(VertexInput, VertexOutput);
			MESH_SHADER_WRITE_VERTEX(VertExportIndex, VertexOutput);
		}

	#if BASEPASS_MESH_SHADER_TG_SIZE == 128
		VertExportIndex += 128;
		LaneVertIndex += 128;
		if (VertexFactoryMeshShaderIsVertexValid(MSIntermediates, VertExportIndex))
		{
			FVertexFactoryInput VertexInput = VertexFactoryMeshShaderGetInput(MSIntermediates, LaneVertIndex);
			FBasePassVSOutput VertexOutput = (FBasePassVSOutput)0;
			Main_Inner(VertexInput, VertexOutput);
			MESH_SHADER_WRITE_VERTEX(VertExportIndex, VertexOutput);
		}
	#endif
	}
}
#endif