// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

template<typename T>
struct TDefaultCompare
{
	int Compare(T a, T b)
	{
		return int(a > b) - int(a < b);
	}
};

// Returns the smallest index in the sorted list such that SearchBuffer[i] >= Key, or BufferOffset + BufferCount if no such index exists.
// The input range must be sorted in ascending order according to the supplied compare.
template<typename TBuffer, typename T, typename TCompare>
T LowerBound(TBuffer SearchBuffer, uint BufferOffset, uint BufferCount, T Key)
{
	TCompare Comp = (TCompare)0;
	uint Lo = BufferOffset;
	uint Hi = BufferOffset + BufferCount;

	while (Lo < Hi)
	{
		const uint Mid = Lo + ((Hi - Lo) >> 1u);
		if (Comp.Compare(SearchBuffer[Mid], Key) < 0)
		{
			Lo = Mid + 1u;
		}
		else
		{
			Hi = Mid;
		}
	}

	return Lo;
}

// Searches the sorted range for a given Key and retrieves the index if found
template<typename TBuffer, typename T, typename TCompare>
bool BinarySearch(TBuffer SearchBuffer, uint BufferOffset, uint BufferCount, T Key, inout uint OutIndex)
{
	if (BufferCount == 0)
	{
		return false;
	}
	const uint EndIndex = BufferOffset + BufferCount;
	OutIndex = LowerBound<TBuffer, T, TCompare>(SearchBuffer, BufferOffset, BufferCount, Key);
	return OutIndex < EndIndex && SearchBuffer[OutIndex] == Key;
}

template<typename TBuffer, typename T, typename TCompare>
bool BinarySearch(TBuffer SearchBuffer, uint BufferOffset, uint BufferCount, T Key)
{
	uint Index;
	return BinarySearch<TBuffer, T, TCompare>(SearchBuffer, BufferOffset, BufferCount, Key, Index);
}

template<typename TBuffer, typename T>
bool BinarySearch(TBuffer SearchBuffer, uint BufferOffset, uint BufferCount, T Key, inout uint OutIndex)
{
	return BinarySearch< TBuffer, T, TDefaultCompare<T> >(SearchBuffer, BufferOffset, BufferCount, Key, OutIndex);
}

template<typename TBuffer, typename T>
bool BinarySearch(TBuffer SearchBuffer, uint BufferOffset, uint BufferCount, T Key)
{
	uint Index;
	return BinarySearch<TBuffer, T>(SearchBuffer, BufferOffset, BufferCount, Key, Index);
}
