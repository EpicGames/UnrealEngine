// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

Texture2D InputTexture;
SamplerState InputSampler;

Texture2D tex0; // Y
Texture2D tex1; // Cr
Texture2D tex2; // Cb
Texture2D tex3; // Alpha
Texture2D tex4; // H
SamplerState samp0;
SamplerState samp1;
SamplerState samp2;
SamplerState samp3;
SamplerState samp4;
float4  consta;
float4  crc; // or hdr
float4  cbc; // or ctcp
float4  adj;
float4  yscale;
float4 xy_xform0;
float4 xy_xform1;
float4 uv_xform0;
float4 uv_xform1;
float4 uv_xform2;

void BinkDrawVS(
	uint i : SV_VertexID,
	out noperspective float4 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	float2 st = float2(i & 1, i >> 1);
	OutPosition.xy = st.x * xy_xform0.xz + st.y * xy_xform0.yw + xy_xform1.xy;
    OutPosition.zw = float2(0.0, 1.0);
	OutUV = st.x * uv_xform0 + st.y * uv_xform1 + uv_xform2;
}

float4 BinkDrawYCbCr(noperspective float4 UV : TEXCOORD0) : SV_Target0
{
	float4 p;

	float y = tex0.Sample( samp0, UV.xy ).r;
	float cr = tex1.Sample( samp1, UV.zw ).r;
	float cb = tex2.Sample( samp2, UV.zw ).r;

	p = y * yscale + cr * crc + cb * cbc + adj;
#if ALPHA
	p.w = tex3.Sample( samp3, UV.xy ).r;
#endif
	p *= consta;
#if SRGB
	p.xyz = p.xyz * (p.xyz * (p.xyz * 0.305306011 + 0.682171111) + 0.012522878);
#endif
	return p;
}

float4 BinkDrawICtCp(noperspective float4 UV : TEXCOORD0) : SV_Target0
{
	float I0 = tex0.Sample( samp0, UV.xy ).r;
	float Ct = tex1.Sample( samp1, UV.zw ).r;
	float Cp = tex2.Sample( samp2, UV.zw ).r;
	float I1 = tex4.Sample( samp4, UV.xy ).r;

	float4 hdr = crc;
	float4 ctcp = cbc;

	Ct = Ct * ctcp.x + ctcp.z;
	Cp = Cp * ctcp.y + ctcp.w;

	float3 LMS = (I0 + I1) * hdr.x;
	LMS += Ct * float3(0.00860903703793281, -0.00860903703793281, 0.56003133571067909);
	LMS += Cp * float3(0.11102962500302593, -0.11102962500302593, -0.32062717498731880);
	LMS = max(LMS, 0);

	// SMPTE-2084 decode
	LMS = pow(LMS, 1/(2523.f/4096*128)); 
	float3 num = max(0, LMS - (3424.f/4096));
	float3 denom = (2413.f/4096*32) - (2392.f/4096*32) * LMS;
	LMS = pow(abs(num / denom), 1/(2610.f/16384));

	float3 rgb = LMS.x * (float3(3.4366066943330793, -0.7913295555989289, -0.0259498996905927) * 125);
	rgb += LMS.y * (float3(-2.5064521186562705, 1.9836004517922909, -0.0989137147117265) * 125);
	rgb += LMS.z * (float3(0.0698454243231915, -0.1922708961933620, 1.1248636144023192) * 125);

	float4 p = float4(rgb,1);

	p.xyz *= hdr.y;  //exposure

#if TONEMAP // Simplified Reinhard
	p.xyz /= p.xyz+1;
	p.xyz *= hdr.z;
#endif

#if ST2084
	// Convert into PQ space
	float3 Lm1 = pow(max(p.xyz*(1.0/125),0), 0.159301758125);
	float3 X = (0.8359375 + 18.8515625 * Lm1) / (1 + 18.6875 * Lm1);
	p.xyz = pow(X, 78.84375);
#endif

#if ALPHA
	p.w = tex3.Sample( samp3, UV.xy ).r;
#endif

	p *= consta;

	return p;
}