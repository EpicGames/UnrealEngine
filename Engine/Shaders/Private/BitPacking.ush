// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

uint3 UnpackToUint3(uint Value, uint3 NumComponentBits)
{
	return uint3(BitFieldExtractU32(Value, NumComponentBits.x, 0),
				 BitFieldExtractU32(Value, NumComponentBits.y, NumComponentBits.x),
				 BitFieldExtractU32(Value, NumComponentBits.z, NumComponentBits.x + NumComponentBits.y));
}

uint4 UnpackToUint4(uint Value, uint4 NumComponentBits)
{
	return uint4(BitFieldExtractU32(Value, NumComponentBits.x, 0),
				 BitFieldExtractU32(Value, NumComponentBits.y, NumComponentBits.x),
				 BitFieldExtractU32(Value, NumComponentBits.z, NumComponentBits.x + NumComponentBits.y),
				 BitFieldExtractU32(Value, NumComponentBits.w, NumComponentBits.x + NumComponentBits.y + NumComponentBits.z));
}

float3 UnpackToFloat3(uint Value, uint3 NumComponentBits)
{
	checkSlow(NumComponentBits.x + NumComponentBits.y + NumComponentBits.z <= 32u);
	return float3(BitFieldExtractFloat(Value, NumComponentBits.x, 0),
				 BitFieldExtractFloat(Value, NumComponentBits.y, NumComponentBits.x),
				 BitFieldExtractFloat(Value, NumComponentBits.z, NumComponentBits.x + NumComponentBits.y));
}

float3 UnpackToFloat3(uint Value, uint BitsPerElement)
{
	return UnpackToFloat3(Value, uint3(BitsPerElement, BitsPerElement, BitsPerElement));
}

uint FloatToUIntScaled(float Value, float Scale)
{
	return (uint)floor(Value * Scale + 0.5f);
}

uint Pack_Float4_To_R10G10B10A2_UNORM(float4 Unpacked)
{
	const float4 UnpackedClamped = saturate(Unpacked);
	uint Packed = ((FloatToUIntScaled(UnpackedClamped.x, 1023))       |
				   (FloatToUIntScaled(UnpackedClamped.y, 1023) << 10) |
				   (FloatToUIntScaled(UnpackedClamped.z, 1023) << 20) |
				   (FloatToUIntScaled(UnpackedClamped.w,    3) << 30));
	return Packed;
}

float4 Unpack_R10G10B10A2_UNORM_To_Float4(uint Packed)
{
	float4 Unpacked;
	Unpacked.x = (float)(((Packed      ) & 0x000003FF)) / 1023;
	Unpacked.y = (float)(((Packed >> 10) & 0x000003FF)) / 1023;
	Unpacked.z = (float)(((Packed >> 20) & 0x000003FF)) / 1023;
	Unpacked.w = (float)(((Packed >> 30) & 0x00000003)) / 3;
	return Unpacked;
}

/**
 * Pack normalized, i.e., range [0..1], float value to uint at a certain number of bits, rounding down.
 */
uint PackNormToUintFloor(float FloatNorm, uint BitsPerElement)
{
	return uint(floor(FloatNorm * float((1u << BitsPerElement) - 1u)));
}

/**
 * Pack normalized, i.e., range [0..1], float value to uint at a certain number of bits, rounding up.
 */
uint PackNormToUintCeil(float FloatNorm, uint BitsPerElement)
{
	return uint(ceil(FloatNorm * float((1u << BitsPerElement) - 1u)));
}

/**
 * Pack uint3 into uint at a certain number of bits.
 */
uint PackToUint(uint3 Packed3, uint3 NumComponentBits)
{
	checkSlow(NumComponentBits.x + NumComponentBits.y + NumComponentBits.z <= 32u);
	return Packed3.x | (Packed3.y << NumComponentBits.x) | (Packed3.z << (NumComponentBits.x + NumComponentBits.y));
}

/**
 * Pack normalized, i.e., range [0..1], float3 value to uint at a certain number of bits per component, rounding down.
 */
uint PackNormToUintFloor(float3 FloatNorm, uint3 NumComponentBits)
{
	return PackToUint(
		uint3(
			PackNormToUintFloor(FloatNorm.x, NumComponentBits.x),
			PackNormToUintFloor(FloatNorm.y, NumComponentBits.y),
			PackNormToUintFloor(FloatNorm.z, NumComponentBits.z)
		),
		NumComponentBits);
}

uint PackNormToUintFloor(float3 FloatNorm, uint BitsPerElement)
{
	return PackNormToUintFloor(FloatNorm, uint3(BitsPerElement, BitsPerElement, BitsPerElement));
}

/**
 * Pack normalized, i.e., range [0..1], float3 value to uint at a certain number of bits per component, rounding up.
 */
uint PackNormToUintCeil(float3 FloatNorm, uint3 NumComponentBits)
{
	return PackToUint(
		uint3(
			PackNormToUintCeil(FloatNorm.x, NumComponentBits.x),
			PackNormToUintCeil(FloatNorm.y, NumComponentBits.y),
			PackNormToUintCeil(FloatNorm.z, NumComponentBits.z)
		), 
		NumComponentBits);
}

uint PackNormToUintCeil(float3 FloatNorm, uint BitsPerElement)
{
	return PackNormToUintCeil(FloatNorm, uint3(BitsPerElement,BitsPerElement,BitsPerElement));
}

// Implement BitStreamReader for ByteAddressBuffer (RO), RWByteAddressBuffer (RW) and dynamic choice (RORW).
struct FBitStreamReaderState
{
	uint	AlignedByteAddress;
	int		BitOffsetFromAddress;

	uint4	BufferBits;
	int		BufferOffset;

	int		CompileTimeMinBufferBits;
	int		CompileTimeMinDwordBits;
	int		CompileTimeMaxRemainingBits;
};

FBitStreamReaderState BitStreamReader_Create_Aligned(uint AlignedByteAddress, uint BitOffset, uint CompileTimeMaxRemainingBits)
{
	FBitStreamReaderState State;

	State.AlignedByteAddress = AlignedByteAddress;
	State.BitOffsetFromAddress = BitOffset;

	State.BufferBits = 0;
	State.BufferOffset = 0;

	State.CompileTimeMinBufferBits = 0;
	State.CompileTimeMinDwordBits = 0;
	State.CompileTimeMaxRemainingBits = CompileTimeMaxRemainingBits;

	return State;
}

FBitStreamReaderState BitStreamReader_Create(uint ByteAddress, uint BitOffset, uint CompileTimeMaxRemainingBits)
{
	uint AlignedByteAddress = ByteAddress & ~3u;
	BitOffset += (ByteAddress & 3u) << 3;
	return BitStreamReader_Create_Aligned(AlignedByteAddress, BitOffset, CompileTimeMaxRemainingBits);
}

#define TYPE_SUFFIX RO
#define INPUT_BUFFER_TYPE ByteAddressBuffer
#include "BitStreamReaderImplementation.ush"
#undef TYPE_SUFFIX
#undef INPUT_BUFFER_TYPE

#define TYPE_SUFFIX RW
#define INPUT_BUFFER_TYPE RWByteAddressBuffer
#include "BitStreamReaderImplementation.ush"
#undef TYPE_SUFFIX
#undef INPUT_BUFFER_TYPE

// Put bits to ByteAddressBuffer at bit offset. NumBits must be <= 31.
void PutBits(RWByteAddressBuffer Output, uint AlignedBaseAddress, uint BitOffset, uint Value, uint NumBits)
{
    uint BitOffsetInDword = (BitOffset & 31u);  // &31 is implicit in shifts
    
    uint Bits = Value << BitOffsetInDword;
    uint Address = AlignedBaseAddress + ((BitOffset >> 5) << 2);
    uint EndBitPos = BitOffsetInDword + NumBits;

    if (EndBitPos >= 32)
    {
        uint Mask = 0xFFFFFFFFu << (EndBitPos & 31u);
        Output.InterlockedAnd(Address + 4, Mask);
        Output.InterlockedOr(Address + 4, Value >> (32 - BitOffsetInDword));
    }

    {
        uint Mask = ~BitFieldMaskU32(NumBits, BitOffset);
        Output.InterlockedAnd(Address, Mask);
        Output.InterlockedOr(Address, Value << BitOffsetInDword);
    }
}

struct FBitStreamWriterState
{
	uint StartAlignedByteAddress;
	uint StartBitOffset;   	
	uint StartBufferBits;

	uint NextAlignedByteAddress;
	uint BitOffset;	
	uint BufferBits;
};

FBitStreamWriterState BitStreamWriter_Create_Aligned(uint AlignedBaseAddressInBytes, uint BitOffset)
{
	FBitStreamWriterState State;

	State.StartAlignedByteAddress = AlignedBaseAddressInBytes + ((BitOffset >> 5) << 2);
	BitOffset &= 31u;
	State.StartBitOffset = BitOffset;
	State.StartBufferBits = 0;		

	State.NextAlignedByteAddress = State.StartAlignedByteAddress + 4u;
	State.BitOffset = BitOffset;	
	State.BufferBits = 0;
	
	return State;
}

void BitStreamWriter_Writer(RWByteAddressBuffer Output, inout FBitStreamWriterState State, uint Value, int NumBits, int CompileTimeMaxBits)
{
	const uint Tmp = Value << (State.BitOffset & 31u);	// & 31u is implicit

    if(State.BitOffset >= 32)
        State.BufferBits |= Tmp;
    else
        State.StartBufferBits |= Tmp;

    const uint NextBitOffset = State.BitOffset + NumBits;
	
	// Overflow to next DWORD?
    if ((State.BitOffset ^ NextBitOffset) >= 32)
    {
		// Keep first DWORD in register, so we can merge it in with atomic later
        if(State.BitOffset >= 32)
        {
            Output.Store(State.NextAlignedByteAddress, State.BufferBits);
            State.NextAlignedByteAddress += 4;
        }
        State.BufferBits = (CompileTimeMaxBits < 32 || (State.BitOffset & 31)) ? (Value >> ((32u - State.BitOffset) & 31u)) : 0u;
    }

	State.BitOffset = NextBitOffset;
}

void BitStreamWriter_Flush(RWByteAddressBuffer Output, inout FBitStreamWriterState State)
{
	// Start
	const uint NumBits = State.BitOffset - State.StartBitOffset;
	uint StartMask =	NumBits >= 32 ? 0xFFFFFFFFu :
						BitFieldMaskU32(NumBits, 0);
	StartMask <<= State.StartBitOffset;
	Output.InterlockedAnd(State.StartAlignedByteAddress, ~StartMask);
	Output.InterlockedOr(State.StartAlignedByteAddress, State.StartBufferBits);

	if (State.BitOffset > 32)
	{
		const uint Mask = BitFieldMaskU32(State.BitOffset & 31u, 0);
		Output.InterlockedAnd(State.NextAlignedByteAddress, ~Mask);
		Output.InterlockedOr(State.NextAlignedByteAddress, State.BufferBits);
	}
}

// Utility functions for packing bits into uints.
// When Position and NumBits can be determined at compile time this should be just as fast as manual bit packing.
uint ReadBits(uint4 Data, inout uint Position, uint NumBits)
{
	uint DwordIndex = Position >> 5;
	uint BitIndex = Position & 31;

	uint Value = Data[DwordIndex] >> BitIndex;
	if (BitIndex + NumBits > 32)
	{
		Value |= Data[DwordIndex + 1] << (32 - BitIndex);
	}

	Position += NumBits;

	uint Mask = NumBits < 32 ? ((1u << NumBits) - 1u) : 0xFFFFFFFFu;
	return Value & Mask;
}

void WriteBits(inout uint4 Data, inout uint Position, uint Value, uint NumBits)
{
	uint DwordIndex = Position >> 5;
	uint BitIndex = Position & 31;

	Data[DwordIndex] |= Value << BitIndex;
	if (BitIndex + NumBits > 32)
	{
		Data[DwordIndex + 1] |= Value >> (32 - BitIndex);
	}

	Position += NumBits;
}
