// Copyright Epic Games, Inc. All Rights Reserved.

// Bit buffer implementation:
// Maintains an internal bit buffer instead of issuing memory loads at every read operation.
// Reads extract the bits from the bottom dword of the bit buffer. Whenever the bottom dword runs out of bits,
// it is refilled by shifting the bit buffer down (v_alignbit). Only when the bit buffer also runs out of bits
// is a memory load issued that then refills the buffer using a single load4.

// If the read sizes are divergent, it is very likely that for a given read at least one thread will need to refill, so
// in the worst case the refill has to happen at every read.
// To mitigate this, all reads have to supply a compile-time constant upper bound to the size of the read.
// By keeping track of these bounds, we can conservatively determine a which reads a refill can possibly be required and only
// emit the refill code in those instances.

// Everything prefixed with CompileTime should be compile-time constant and generate no code.
// We unfortunately have no way to enforce this.


// BitStreamReader
// Helper 'class' for efficiently parsing bit streams of arbitrary length.
#define CONCAT2(A, B) A##B
#define CONCAT(A, B) CONCAT2(A, B)

//TODO: Rewrite this with templates, so CompileMaxBits can be proper compile-time
uint CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)
	(INPUT_BUFFER_TYPE InputBuffer, inout FBitStreamReaderState State, int NumBits, int CompileTimeMaxBits)
{
	if (CompileTimeMaxBits > State.CompileTimeMinBufferBits)
	{
		// BitBuffer could be out of bits: Reload.

		// Add cumulated offset since last refill. No need to update at every read.
		State.BitOffsetFromAddress += State.BufferOffset;	
		uint Address = State.AlignedByteAddress + ((State.BitOffsetFromAddress >> 5) << 2);

		uint4 Data = InputBuffer.Load4(Address);

		// Shift bits down to align
		State.BufferBits.x												= BitAlignU32(Data.y,	Data.x,	State.BitOffsetFromAddress); // BitOffsetFromAddress implicitly &31
		if (State.CompileTimeMaxRemainingBits > 32) State.BufferBits.y	= BitAlignU32(Data.z,	Data.y,	State.BitOffsetFromAddress); // BitOffsetFromAddress implicitly &31
		if (State.CompileTimeMaxRemainingBits > 64) State.BufferBits.z	= BitAlignU32(Data.w,	Data.z,	State.BitOffsetFromAddress); // BitOffsetFromAddress implicitly &31
		if (State.CompileTimeMaxRemainingBits > 96) State.BufferBits.w	= BitAlignU32(0,		Data.w,	State.BitOffsetFromAddress); // BitOffsetFromAddress implicitly &31

		State.BufferOffset = 0;

		State.CompileTimeMinDwordBits	= min(32, State.CompileTimeMaxRemainingBits);
		State.CompileTimeMinBufferBits	= min(97, State.CompileTimeMaxRemainingBits);	// Up to 31 bits wasted to alignment
	}
	else if (CompileTimeMaxBits > State.CompileTimeMinDwordBits)
	{
		// Bottom dword could be out of bits: Shift down.
		State.BitOffsetFromAddress += State.BufferOffset;

		// Workaround for BitAlignU32(x, y, 32) returning x instead of y.
		// In the common case where State.CompileTimeMinDwordBits != 0, this will be optimized to just BitAlignU32.
		// TODO: Can we get rid of this special case?
		const bool bOffset32 = State.CompileTimeMinDwordBits == 0 && State.BufferOffset == 32;

		State.BufferBits.x											= bOffset32 ? State.BufferBits.y :	BitAlignU32(State.BufferBits.y, State.BufferBits.x, State.BufferOffset);
		if (State.CompileTimeMinBufferBits > 32) State.BufferBits.y	= bOffset32 ? State.BufferBits.z :	BitAlignU32(State.BufferBits.z, State.BufferBits.y, State.BufferOffset);
		if (State.CompileTimeMinBufferBits > 64) State.BufferBits.z	= bOffset32 ? State.BufferBits.w :	BitAlignU32(State.BufferBits.w, State.BufferBits.z, State.BufferOffset);
		if (State.CompileTimeMinBufferBits > 96) State.BufferBits.w	= bOffset32 ? 0u :					BitAlignU32(0,					State.BufferBits.w, State.BufferOffset);
	
		State.BufferOffset = 0;

		State.CompileTimeMinDwordBits = min(32, State.CompileTimeMaxRemainingBits);
	}

	const bool bNumBits32 = (CompileTimeMaxBits >= 32) && NumBits == 32;	// This will be optimized away unless CompileTimeMaxBits is explicitly set to 32.
	const uint Result = bNumBits32 ? State.BufferBits.x : BitFieldExtractU32(State.BufferBits.x, NumBits, State.BufferOffset); // BufferOffset implicitly &31
	
	State.BufferOffset += NumBits;
	State.CompileTimeMinBufferBits    -= CompileTimeMaxBits;
	State.CompileTimeMinDwordBits     -= CompileTimeMaxBits;
	State.CompileTimeMaxRemainingBits -= CompileTimeMaxBits;

	return Result;
}

uint2 CONCAT(BitStreamReader_Read2_, TYPE_SUFFIX)
	(INPUT_BUFFER_TYPE InputBuffer, inout FBitStreamReaderState State, int2 NumBits, int2 CompileTimeMaxBits)
{
	uint ResultX = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.x, CompileTimeMaxBits.x);
	uint ResultY = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.y, CompileTimeMaxBits.y);
	return uint2(ResultX, ResultY);
}

uint3 CONCAT(BitStreamReader_Read3_, TYPE_SUFFIX)
	(INPUT_BUFFER_TYPE InputBuffer, inout FBitStreamReaderState State, int3 NumBits, int3 CompileTimeMaxBits)
{
	uint ResultX = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.x, CompileTimeMaxBits.x);
	uint ResultY = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.y, CompileTimeMaxBits.y);
	uint ResultZ = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.z, CompileTimeMaxBits.z);
	return uint3(ResultX, ResultY, ResultZ);
}

uint4 CONCAT(BitStreamReader_Read4_, TYPE_SUFFIX)
	(INPUT_BUFFER_TYPE InputBuffer, inout FBitStreamReaderState State, int4 NumBits, int4 CompileTimeMaxBits)
{
	uint ResultX = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.x, CompileTimeMaxBits.x);
	uint ResultY = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.y, CompileTimeMaxBits.y);
	uint ResultZ = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.z, CompileTimeMaxBits.z);
	uint ResultW = CONCAT(BitStreamReader_Read_, TYPE_SUFFIX)(InputBuffer, State, NumBits.w, CompileTimeMaxBits.w);
	return uint4(ResultX, ResultY, ResultZ, ResultW);
}

#undef CONCAT
#undef CONCAT2