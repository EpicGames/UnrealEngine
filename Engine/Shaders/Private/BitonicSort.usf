// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	BitonicSort.usf: compute-shader bitonic sort. Local sort/merge use a three-tier
	BitonicSubSteps (shared memory / wave shuffle / register swap by stride);
	global merge is in-place on RWBuffers.
==============================================================================*/

#include "Common.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"

#define BLOCK_SIZE (BITONIC_THREAD_COUNT * BITONIC_ELEMENTS_PER_THREAD)

// Reserved as the pad sentinel. Callers must not use 0xFFFFFFFF as a real key.
// Real keys are clamped to SENTINEL_KEY-1 on load to defend against violations.
#define SENTINEL_KEY 0xFFFFFFFF

#define SUBGROUP_SIZE 32
#define LARGE_STRIDE_THRESHOLD (BITONIC_ELEMENTS_PER_THREAD * SUBGROUP_SIZE)

// Bank-conflict-free shared-memory indexing: pad 1 word per 32 elements.
#define SHARED_ARRAY_SIZE (BLOCK_SIZE + (BLOCK_SIZE >> 5))

uint SharedIdx(uint Idx)
{
	return Idx + (Idx >> 5);
}

Buffer<uint> InKeys;
// InValues is a UAV so the caller can alias it with OutValues on single-pass sorts.
RWBuffer<uint> InValues;
RWBuffer<uint> OutKeys;
RWBuffer<uint> OutValues;

#if BITONIC_LOCAL_SORT || BITONIC_LOCAL_MERGE

groupshared uint SharedKeys[SHARED_ARRAY_SIZE];
groupshared uint SharedValues[SHARED_ARRAY_SIZE];

void SortPair(uint IdxA, uint IdxB, bool bAscending)
{
	uint PA = SharedIdx(IdxA);
	uint PB = SharedIdx(IdxB);

	uint KeyA = SharedKeys[PA];
	uint KeyB = SharedKeys[PB];
	uint ValA = SharedValues[PA];
	uint ValB = SharedValues[PB];

	bool bSwap = (KeyA > KeyB) == bAscending;
	SharedKeys[PA]   = bSwap ? KeyB : KeyA;
	SharedKeys[PB]   = bSwap ? KeyA : KeyB;
	SharedValues[PA] = bSwap ? ValB : ValA;
	SharedValues[PB] = bSwap ? ValA : ValB;
}

// ConstantAscending: 1=ascending, 0=descending, -1=per-element ((LocalIdx & SortBlockSize)==0).
void BitonicSubSteps(uint ThreadId, uint StartStride, uint SortBlockSize, int ConstantAscending)
{
	uint e, CompStride;

	for (CompStride = StartStride; CompStride >= LARGE_STRIDE_THRESHOLD; CompStride >>= 1)
	{
		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			uint Idx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
			uint Partner = Idx ^ CompStride;
			if (Partner > Idx)
			{
				bool bAsc = (ConstantAscending >= 0)
					? (bool)ConstantAscending
					: ((Idx & SortBlockSize) == 0);
				SortPair(Idx, Partner, bAsc);
			}
		}
		GroupMemoryBarrierWithGroupSync();
	}

	uint Keys[BITONIC_ELEMENTS_PER_THREAD];
	uint Vals[BITONIC_ELEMENTS_PER_THREAD];

	[unroll]
	for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
	{
		uint Idx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
		Keys[e] = SharedKeys[SharedIdx(Idx)];
		Vals[e] = SharedValues[SharedIdx(Idx)];
	}

	uint MaxRegStride = min(StartStride, (uint)(LARGE_STRIDE_THRESHOLD / 2));

	// Tier 2: medium strides
	// Unrolled so CompStride / LaneXorMask is a compile-time constant 
	//WaveBroadcast lower to GCN's ds_swizzle_b32 (immediate-XOR-only instruction).
	#if SUBGROUP_SIZE != 32
		#error "TIER2_STEP_COUNT assumes SUBGROUP_SIZE=32 -- recompute as log2(SUBGROUP_SIZE/2)+1 and update this define"
	#endif
	#define TIER2_STEP_COUNT 5

	[unroll]
	for (uint T2Step = 0; T2Step < TIER2_STEP_COUNT; ++T2Step)
	{
		const uint T2CompStride = (uint)(LARGE_STRIDE_THRESHOLD / 2) >> T2Step;
		if (T2CompStride > MaxRegStride)
		{
			continue;
		}

#if BITONIC_USE_WAVE_OPS && PLATFORM_SUPPORTS_WAVE_BROADCAST
		// Single-step wave-butterfly exchange with compile-time XOR mask.
		const FWaveBroadcastSettings ShuffleSettings = InitWaveXorButterfly(T2CompStride / BITONIC_ELEMENTS_PER_THREAD);

		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			uint Idx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
			bool bAsc = (ConstantAscending >= 0)
				? (bool)ConstantAscending
				: ((Idx & SortBlockSize) == 0);

			uint PartnerKey = WaveBroadcast(ShuffleSettings, Keys[e]);
			uint PartnerVal = WaveBroadcast(ShuffleSettings, Vals[e]);

			bool bIsLow = (Idx & T2CompStride) == 0;
			bool bSwap = bIsLow
				? ((Keys[e] > PartnerKey) == bAsc)
				: ((Keys[e] < PartnerKey) == bAsc);

			if (bSwap)
			{
				Keys[e] = PartnerKey;
				Vals[e] = PartnerVal;
			}
		}
#else
		// Shared-memory fallback without WAVE_BROADCAST: flush, compare-swap, reload.
		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			SharedKeys[SharedIdx(ThreadId * BITONIC_ELEMENTS_PER_THREAD + e)] = Keys[e];
			SharedValues[SharedIdx(ThreadId * BITONIC_ELEMENTS_PER_THREAD + e)] = Vals[e];
		}
		GroupMemoryBarrierWithGroupSync();

		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			uint SIdx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
			uint SPartner = SIdx ^ T2CompStride;
			if (SPartner > SIdx)
			{
				bool SAsc = (ConstantAscending >= 0)
					? (bool)ConstantAscending
					: ((SIdx & SortBlockSize) == 0);
				SortPair(SIdx, SPartner, SAsc);
			}
		}
		GroupMemoryBarrierWithGroupSync();

		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			Keys[e] = SharedKeys[SharedIdx(ThreadId * BITONIC_ELEMENTS_PER_THREAD + e)];
			Vals[e] = SharedValues[SharedIdx(ThreadId * BITONIC_ELEMENTS_PER_THREAD + e)];
		}
#endif
	}

	#undef TIER2_STEP_COUNT

	// Tier 3: stride < EPT, partners in the same thread's registers. Always runs (StartStride >= 1).
	[unroll]
	for (CompStride = BITONIC_ELEMENTS_PER_THREAD / 2; CompStride >= 1; CompStride >>= 1)
	{
		[unroll]
		for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
		{
			uint Idx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
			bool bAsc = (ConstantAscending >= 0)
				? (bool)ConstantAscending
				: ((Idx & SortBlockSize) == 0);

			uint PartnerE = e ^ CompStride;
			if (PartnerE > e)
			{
				if ((Keys[e] > Keys[PartnerE]) == bAsc)
				{
					uint TmpK = Keys[e]; Keys[e] = Keys[PartnerE]; Keys[PartnerE] = TmpK;
					uint TmpV = Vals[e]; Vals[e] = Vals[PartnerE]; Vals[PartnerE] = TmpV;
				}
			}
		}
	}

	[unroll]
	for (e = 0; e < BITONIC_ELEMENTS_PER_THREAD; ++e)
	{
		uint Idx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + e;
		SharedKeys[SharedIdx(Idx)] = Keys[e];
		SharedValues[SharedIdx(Idx)] = Vals[e];
	}
	GroupMemoryBarrierWithGroupSync();
}

#endif // BITONIC_LOCAL_SORT || BITONIC_LOCAL_MERGE

#if BITONIC_LOCAL_SORT

#if BITONIC_USE_WAVE_OPS && COMPILER_SUPPORTS_WAVE_SIZE
WAVESIZE(32)
#endif
[numthreads(BITONIC_THREAD_COUNT, 1, 1)]
void BitonicSort_LocalSort(uint3 GroupThreadId : SV_GroupThreadID, uint3 GroupId : SV_GroupID)
{
	const uint ThreadId = GroupThreadId.x;
	const uint GroupOffset = GroupId.x * BLOCK_SIZE;

	[unroll]
	for (uint i = 0; i < BITONIC_ELEMENTS_PER_THREAD; ++i)
	{
		uint LocalIdx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + i;
		uint GlobalIdx = GroupOffset + LocalIdx;
		uint PIdx = SharedIdx(LocalIdx);
		if (GlobalIdx < BitonicSortUB.BitonicSort_Count)
		{
			// Clamp below SENTINEL — bitonic is unstable, so a real 0xFFFFFFFF could tie with a pad sentinel and get permuted into the [Count, PaddedCount) tail.
			SharedKeys[PIdx] = min(InKeys[GlobalIdx], SENTINEL_KEY - 1u);
			SharedValues[PIdx] = InValues[GlobalIdx];
		}
		else
		{
			SharedKeys[PIdx] = SENTINEL_KEY;
			SharedValues[PIdx] = 0;
		}
	}
	GroupMemoryBarrierWithGroupSync();

	// log2(BLOCK_SIZE) stages; last stage alternates direction per block to produce bitonic sequences for the global merge, intermediates use per-element direction.
	for (uint SortBlockSize = 2; SortBlockSize <= BLOCK_SIZE; SortBlockSize <<= 1)
	{
		if (SortBlockSize == BLOCK_SIZE)
		{
			int BlockAscending = ((GroupId.x & 1) == 0) ? 1 : 0;
			BitonicSubSteps(ThreadId, SortBlockSize >> 1, SortBlockSize, BlockAscending);
		}
		else
		{
			BitonicSubSteps(ThreadId, SortBlockSize >> 1, SortBlockSize, -1);
		}
	}

	// Unconditional write: dispatch covers exactly PaddedCount.
	[unroll]
	for (uint j = 0; j < BITONIC_ELEMENTS_PER_THREAD; ++j)
	{
		uint LocalIdx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + j;
		uint GlobalIdx = GroupOffset + LocalIdx;
		OutKeys[GlobalIdx] = SharedKeys[SharedIdx(LocalIdx)];
		OutValues[GlobalIdx] = SharedValues[SharedIdx(LocalIdx)];
	}
}

#endif

#if BITONIC_GLOBAL_MERGE

RWBuffer<uint> InOutKeys;
RWBuffer<uint> InOutValues;

#define GLOBAL_MERGE_EPT BITONIC_GLOBAL_MERGE_EPT

[numthreads(BITONIC_THREAD_COUNT, 1, 1)]
void BitonicSort_GlobalMerge(uint3 GroupThreadId : SV_GroupThreadID, uint3 GroupId : SV_GroupID)
{
	const uint GlobalThread = GroupId.x * BITONIC_THREAD_COUNT + GroupThreadId.x;
	const uint HalfStep = BitonicSortUB.BitonicSort_StepBlockSize >> 1;

	[unroll]
	for (uint e = 0; e < GLOBAL_MERGE_EPT; ++e)
	{
		const uint PairIndex = GlobalThread * GLOBAL_MERGE_EPT + e;

		// Power-of-2 div/mod via bitwise ops (~3 ops vs ~20 for integer division).
		const uint Offset = PairIndex & (HalfStep - 1);
		const uint LeftIdx = (PairIndex << 1) - Offset;
		const uint RightIdx = LeftIdx + HalfStep;

		if (RightIdx < BitonicSortUB.BitonicSort_PaddedCount)
		{
			bool bAscending = ((LeftIdx & BitonicSortUB.BitonicSort_StageBlockSize) == 0);

			uint KeyL = InOutKeys[LeftIdx];
			uint ValL = InOutValues[LeftIdx];
			uint KeyR = InOutKeys[RightIdx];
			uint ValR = InOutValues[RightIdx];

			if (bAscending == (KeyL > KeyR))
			{
				InOutKeys[LeftIdx] = KeyR;
				InOutKeys[RightIdx] = KeyL;
				InOutValues[LeftIdx] = ValR;
				InOutValues[RightIdx] = ValL;
			}
		}
	}
}

#endif

#if BITONIC_LOCAL_MERGE

#if BITONIC_USE_WAVE_OPS && COMPILER_SUPPORTS_WAVE_SIZE
WAVESIZE(32)
#endif
[numthreads(BITONIC_THREAD_COUNT, 1, 1)]
void BitonicSort_LocalMerge(uint3 GroupThreadId : SV_GroupThreadID, uint3 GroupId : SV_GroupID)
{
	const uint ThreadId = GroupThreadId.x;
	const uint GroupOffset = GroupId.x * BLOCK_SIZE;

	// Unconditional load: after LocalSort all PaddedCount positions hold real data or sentinels.
	[unroll]
	for (uint i = 0; i < BITONIC_ELEMENTS_PER_THREAD; ++i)
	{
		uint LocalIdx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + i;
		uint GlobalIdx = GroupOffset + LocalIdx;
		uint PIdx = SharedIdx(LocalIdx);
		SharedKeys[PIdx] = InKeys[GlobalIdx];
		SharedValues[PIdx] = InValues[GlobalIdx];
	}
	GroupMemoryBarrierWithGroupSync();

	int Ascending = ((GroupOffset & BitonicSortUB.BitonicSort_StageBlockSize) == 0) ? 1 : 0;

	BitonicSubSteps(ThreadId, BitonicSortUB.BitonicSort_StepBlockSize >> 1, BitonicSortUB.BitonicSort_StageBlockSize, Ascending);

	// Unconditional write: dispatch covers exactly PaddedCount.
	[unroll]
	for (uint j = 0; j < BITONIC_ELEMENTS_PER_THREAD; ++j)
	{
		uint LocalIdx = ThreadId * BITONIC_ELEMENTS_PER_THREAD + j;
		uint GlobalIdx = GroupOffset + LocalIdx;
		OutKeys[GlobalIdx] = SharedKeys[SharedIdx(LocalIdx)];
		OutValues[GlobalIdx] = SharedValues[SharedIdx(LocalIdx)];
	}
}

#endif
