// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlockCompressionCommon.ush:
	Helpers for compute shader block texture compression
=============================================================================*/

#pragma once

#ifndef BLOCK_COMPRESS_SRC_TEXTURE_ARRAY
#define BLOCK_COMPRESS_SRC_TEXTURE_ARRAY 0
#endif

#if BLOCK_COMPRESS_SRC_TEXTURE_ARRAY
#define TextureType Texture2DArray
#define CoordType float3
#else
#define TextureType Texture2D
#define CoordType float2
#endif

// Read a 4x4 color block ready for BC compression
void ReadBlockRGB(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float3 Block[16])
{
	{
		float4 Red = SourceTexture.GatherRed(TextureSampler, UV);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UV);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UV);
		Block[0] = float3(Red[3], Green[3], Blue[3]);
		Block[1] = float3(Red[2], Green[2], Blue[2]);
		Block[4] = float3(Red[0], Green[0], Blue[0]);
		Block[5] = float3(Red[1], Green[1], Blue[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		Block[2] = float3(Red[3], Green[3], Blue[3]);
		Block[3] = float3(Red[2], Green[2], Blue[2]);
		Block[6] = float3(Red[0], Green[0], Blue[0]);
		Block[7] = float3(Red[1], Green[1], Blue[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		Block[8] = float3(Red[3], Green[3], Blue[3]);
		Block[9] = float3(Red[2], Green[2], Blue[2]);
		Block[12] = float3(Red[0], Green[0], Blue[0]);
		Block[13] = float3(Red[1], Green[1], Blue[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		Block[10] = float3(Red[3], Green[3], Blue[3]);
		Block[11] = float3(Red[2], Green[2], Blue[2]);
		Block[14] = float3(Red[0], Green[0], Blue[0]);
		Block[15] = float3(Red[1], Green[1], Blue[1]);
	}
}

// Read a 4x4 alpha block ready for BC compression
void ReadBlockAlpha(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float Block[16])
{
	{
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UV);
		Block[0] = Alpha[3];
		Block[1] = Alpha[2];
		Block[4] = Alpha[0];
		Block[5] = Alpha[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);
		Block[2] = Alpha[3];
		Block[3] = Alpha[2];
		Block[6] = Alpha[0];
		Block[7] = Alpha[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);
		Block[8] = Alpha[3];
		Block[9] = Alpha[2];
		Block[12] = Alpha[0];
		Block[13] = Alpha[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);
		Block[10] = Alpha[3];
		Block[11] = Alpha[2];
		Block[14] = Alpha[0];
		Block[15] = Alpha[1];
	}
}

void ReadBlockRGBA(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float4 Block[16])
{
	{
		float4 Red = SourceTexture.GatherRed(TextureSampler, UV);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UV);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UV);
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UV);

		Block[0] = float4(Red[3], Green[3], Blue[3], Alpha[3]);
		Block[1] = float4(Red[2], Green[2], Blue[2], Alpha[2]);
		Block[4] = float4(Red[0], Green[0], Blue[0], Alpha[0]);
		Block[5] = float4(Red[1], Green[1], Blue[1], Alpha[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);

		Block[2] = float4(Red[3], Green[3], Blue[3], Alpha[3]);
		Block[3] = float4(Red[2], Green[2], Blue[2], Alpha[2]);
		Block[6] = float4(Red[0], Green[0], Blue[0], Alpha[0]);
		Block[7] = float4(Red[1], Green[1], Blue[1], Alpha[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);
		Block[8] = float4(Red[3], Green[3], Blue[3], Alpha[3]);
		Block[9] = float4(Red[2], Green[2], Blue[2], Alpha[2]);
		Block[12] = float4(Red[0], Green[0], Blue[0], Alpha[0]);
		Block[13] = float4(Red[1], Green[1], Blue[1], Alpha[1]);
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		float4 Alpha = SourceTexture.GatherAlpha(TextureSampler, UVOffset);
		Block[10] = float4(Red[3], Green[3], Blue[3], Alpha[3]);
		Block[11] = float4(Red[2], Green[2], Blue[2], Alpha[2]);
		Block[14] = float4(Red[0], Green[0], Blue[0], Alpha[0]);
		Block[15] = float4(Red[1], Green[1], Blue[1], Alpha[1]);
	}
}

// Read a 4x4 color of channel X block ready for BC compression
void ReadBlockX(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float Block[16])
{
	{
		float4 Red = SourceTexture.GatherRed(TextureSampler, UV);
		Block[0] = Red[3];
		Block[1] = Red[2];
		Block[4] = Red[0];
		Block[5] = Red[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		Block[2] = Red[3];
		Block[3] = Red[2];
		Block[6] = Red[0];
		Block[7] = Red[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		Block[8] = Red[3];
		Block[9] = Red[2];
		Block[12] = Red[0];
		Block[13] = Red[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		Block[10] = Red[3];
		Block[11] = Red[2];
		Block[14] = Red[0];
		Block[15] = Red[1];
	}
}

// Read a 4x4 block of XY channels from a normal texture ready for BC5 compression.
void ReadBlockXY(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float BlockX[16], out float BlockY[16])
{
	{
		float4 Red = SourceTexture.GatherRed(TextureSampler, UV);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UV);
		BlockX[0] = Red[3]; BlockY[0] = Green[3];
		BlockX[1] = Red[2]; BlockY[1] = Green[2];
		BlockX[4] = Red[0]; BlockY[4] = Green[0];
		BlockX[5] = Red[1]; BlockY[5] = Green[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		BlockX[2] = Red[3]; BlockY[2] = Green[3];
		BlockX[3] = Red[2]; BlockY[3] = Green[2];
		BlockX[6] = Red[0]; BlockY[6] = Green[0];
		BlockX[7] = Red[1]; BlockY[7] = Green[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		BlockX[8] = Red[3]; BlockY[8] = Green[3];
		BlockX[9] = Red[2]; BlockY[9] = Green[2];
		BlockX[12] = Red[0]; BlockY[12] = Green[0];
		BlockX[13] = Red[1]; BlockY[13] = Green[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		BlockX[10] = Red[3]; BlockY[10] = Green[3];
		BlockX[11] = Red[2]; BlockY[11] = Green[2];
		BlockX[14] = Red[0]; BlockY[14] = Green[0];
		BlockX[15] = Red[1]; BlockY[15] = Green[1];
	}
}

// Read a 4x4 block of XYA channels ready for BC4/BC5 compression.
void ReadBlockXYA(TextureType<float4> SourceTexture, SamplerState TextureSampler, CoordType UV, float2 TexelUVSize, out float BlockX[16], out float BlockY[16], out float BlockA[16])
{
	{
		float4 Red = SourceTexture.GatherRed(TextureSampler, UV);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UV);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UV);
		BlockX[0] = Red[3]; BlockY[0] = Green[3]; BlockA[0] = Blue[3];
		BlockX[1] = Red[2]; BlockY[1] = Green[2]; BlockA[1] = Blue[2];
		BlockX[4] = Red[0]; BlockY[4] = Green[0]; BlockA[4] = Blue[0];
		BlockX[5] = Red[1]; BlockY[5] = Green[1]; BlockA[5] = Blue[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.x += 2.f * TexelUVSize.x;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		BlockX[2] = Red[3]; BlockY[2] = Green[3]; BlockA[2] = Blue[3];
		BlockX[3] = Red[2]; BlockY[3] = Green[2]; BlockA[3] = Blue[2];
		BlockX[6] = Red[0]; BlockY[6] = Green[0]; BlockA[6] = Blue[0];
		BlockX[7] = Red[1]; BlockY[7] = Green[1]; BlockA[7] = Blue[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.y += 2.f * TexelUVSize.y;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		BlockX[8] = Red[3]; BlockY[8] = Green[3]; BlockA[8] = Blue[3];
		BlockX[9] = Red[2]; BlockY[9] = Green[2]; BlockA[9] = Blue[2];
		BlockX[12] = Red[0]; BlockY[12] = Green[0]; BlockA[12] = Blue[0];
		BlockX[13] = Red[1]; BlockY[13] = Green[1]; BlockA[13] = Blue[1];
	}
	{
		CoordType UVOffset = UV;
		UVOffset.xy += 2.f * TexelUVSize;
		float4 Red = SourceTexture.GatherRed(TextureSampler, UVOffset);
		float4 Green = SourceTexture.GatherGreen(TextureSampler, UVOffset);
		float4 Blue = SourceTexture.GatherBlue(TextureSampler, UVOffset);
		BlockX[10] = Red[3]; BlockY[10] = Green[3]; BlockA[10] = Blue[3];
		BlockX[11] = Red[2]; BlockY[11] = Green[2]; BlockA[11] = Blue[2];
		BlockX[14] = Red[0]; BlockY[14] = Green[0]; BlockA[14] = Blue[0];
		BlockX[15] = Red[1]; BlockY[15] = Green[1]; BlockA[15] = Blue[1];
	}
}

#undef TextureType
#undef CoordType
