// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

uint bProcessAlpha;

Texture2D KernelSpatialTexture;
StructuredBuffer<FBloomKernelInfo> KernelConstantsBuffer;
RWTexture2D<float4> ClampedKernelSpatialOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	FBloomKernelInfo OriginalKernelInfo = KernelConstantsBuffer[0];

	uint2 PixelCoord = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);

	float4 KernelTexel = KernelSpatialTexture[PixelCoord];

	float4 ClampedKernel = clamp(KernelTexel, 0.0, OriginalKernelInfo.MaxScatterDispersionEnergy);
	FLATTEN
	if (!bProcessAlpha)
	{
		ClampedKernel.a = ClampedKernel.b;
	}

	ClampedKernelSpatialOutput[PixelCoord] = ClampedKernel;
}
