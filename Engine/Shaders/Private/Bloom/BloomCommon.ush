// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"


//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


//------------------------------------------------------- STRUCTURES

struct FBloomKernelInfo
{
	// Total energy at the center of the kernel.
	float4 CenterEnergy;
	
	// Total energy of the bloom.
	float4 ScatterDispersionEnergy;

	// Highest energy per pixel that should clamp with to remove the kernel center
	float4 MaxScatterDispersionEnergy;

	// Pixel coordinate of the center of the kernel
	uint2 CenterPixelCoord;

	// Some padding to make vulkan happy
	uint2 _Padding;
};
