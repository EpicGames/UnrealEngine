// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

uint2 KernelSpatialTextureSize;
Texture2D KernelSpatialTexture;
SamplerState KernelSpatialSampler;

RWTexture2D<float4> DownsampleKernelSpatialOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 OutputPixelCoord = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);

	float2 TextureUV = (float2(OutputPixelCoord) * 2.0 + 1.0) / float2(KernelSpatialTextureSize);

	float2 Offset = 0.7 / float2(KernelSpatialTextureSize);

	float4 KernelColor = 0.0;
	KernelColor += 0.25 * KernelSpatialTexture.SampleLevel(KernelSpatialSampler, TextureUV + float2(+1, +1) * Offset, 0);
	KernelColor += 0.25 * KernelSpatialTexture.SampleLevel(KernelSpatialSampler, TextureUV + float2(+1, -1) * Offset, 0);
	KernelColor += 0.25 * KernelSpatialTexture.SampleLevel(KernelSpatialSampler, TextureUV + float2(-1, -1) * Offset, 0);
	KernelColor += 0.25 * KernelSpatialTexture.SampleLevel(KernelSpatialSampler, TextureUV + float2(-1, +1) * Offset, 0);

	DownsampleKernelSpatialOutput[OutputPixelCoord] = KernelColor;
}
