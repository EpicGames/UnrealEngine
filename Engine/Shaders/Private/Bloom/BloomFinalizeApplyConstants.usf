// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- PARAMETERS

float ScatterDispersionIntensity;

StructuredBuffer<FBloomKernelInfo> KernelConstantsBuffer;
RWStructuredBuffer<float4> SceneColorApplyOutput;
RWStructuredBuffer<float4> FFTMulitplyOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(1, 1, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	FBloomKernelInfo OriginalKernelInfo = KernelConstantsBuffer[0];

	// Compute the tint of the kernel
	float4 Tint;
	{
		float3 RefTotalEnergy = OriginalKernelInfo.CenterEnergy.rgb + OriginalKernelInfo.ScatterDispersionEnergy.rgb;

		float MaxEnergy = max3(RefTotalEnergy.r, RefTotalEnergy.g, RefTotalEnergy.b);

		Tint = float4(saturate(RefTotalEnergy / MaxEnergy), 1.0);
	}

	float4 OriginalTotalEnergy = OriginalKernelInfo.CenterEnergy + OriginalKernelInfo.ScatterDispersionEnergy;
	
	float4 FinalScatterDispersionEnergy = OriginalKernelInfo.ScatterDispersionEnergy * ScatterDispersionIntensity;
	
	float4 FinalCenterEnergy = OriginalTotalEnergy - FinalScatterDispersionEnergy;

	SceneColorApplyOutput[0] = Tint * saturate(FinalCenterEnergy / OriginalTotalEnergy);
	FFTMulitplyOutput[0] = Tint * saturate(FinalScatterDispersionEnergy / OriginalTotalEnergy);
}
