// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONSTANTS

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

/*
uint
uint
float
uint
* */
uint2 KernelSpatialTextureSize;
Texture2D KernelSpatialTexture;

RWCoherentStructuredBuffer(uint) KernelCenterCoordOutput;


//------------------------------------------------------- LDS

groupshared uint MaxLuminance;
groupshared uint2 MaxLuminancePixelPos;
groupshared uint CurrentMaxLuminance;
groupshared uint bDone;
groupshared uint PrevLock;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	// Init LDS
	if (GroupThreadIndex == 0)
	{
		MaxLuminance = 0;
		bDone = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	uint2 InputPixelPos = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);

	float4 TexelColor = KernelSpatialTexture[InputPixelPos];

	bool bValidPixel = all(InputPixelPos < KernelSpatialTextureSize);

	if (!bValidPixel)
	{
		TexelColor = 0.0;
	}

	// Locate the position of the brightest pixel
	{
		float TexelLuminance = Luminance(TexelColor.rgb);

		InterlockedMax(MaxLuminance, asuint(TexelLuminance));

		GroupMemoryBarrierWithGroupSync();

		if (MaxLuminance == asuint(TexelLuminance))
		{
			MaxLuminancePixelPos = InputPixelPos;
		}
	}
	
	GroupMemoryBarrierWithGroupSync();

	if (MaxLuminance == 0)
	{
		return;
	}

	while (bDone == 0)
	{
		if (GroupThreadIndex == 0)
		{
			CurrentMaxLuminance = KernelCenterCoordOutput[2];
		}
		
		GroupMemoryBarrierWithGroupSync();

		if (MaxLuminance > CurrentMaxLuminance)
		{
			if (GroupThreadIndex == 0)
			{
				uint LocalPrevLock;
				InterlockedCompareExchange(
					KernelCenterCoordOutput[3],
					uint(0),
					uint(1),
					/* out */ LocalPrevLock);

				PrevLock = LocalPrevLock;
			}
			
			GroupMemoryBarrierWithGroupSync();

			if (PrevLock == 0)
			{
				AllMemoryBarrier();

				if (GroupThreadIndex == 0 && MaxLuminance > KernelCenterCoordOutput[2])
				{
					KernelCenterCoordOutput[0] = MaxLuminancePixelPos.x;
					KernelCenterCoordOutput[1] = MaxLuminancePixelPos.y;
					KernelCenterCoordOutput[2] = MaxLuminance;
				}

				AllMemoryBarrier();
				
				if (GroupThreadIndex == 0)
				{
					uint Unused;
					InterlockedExchange(KernelCenterCoordOutput[3], 0, Unused);
				}
				bDone += 1;
			}
		}
		else
		{
			bDone += 1;
		}
		
		GroupMemoryBarrierWithGroupSync();
	}
}
