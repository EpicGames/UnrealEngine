// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- PARAMETERS

StructuredBuffer<uint> KernelCenterCoordBuffer;
StructuredBuffer<float4> KernelCenterEnergyBuffer;
StructuredBuffer<float4> MaxScatterDispersionBuffer;
Texture2D ScatterDispersionTexture;

RWStructuredBuffer<FBloomKernelInfo> KernelConstantsOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(1, 1, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	KernelConstantsOutput[0].CenterEnergy = KernelCenterEnergyBuffer[0];
	KernelConstantsOutput[0].ScatterDispersionEnergy = ScatterDispersionTexture[uint2(0, 0)];
	KernelConstantsOutput[0].MaxScatterDispersionEnergy = MaxScatterDispersionBuffer[0];
	KernelConstantsOutput[0].CenterPixelCoord = uint2(KernelCenterCoordBuffer[0], KernelCenterCoordBuffer[1]);
}
