// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 64


//------------------------------------------------------- PARAMETERS

uint SurveyReduceOp;
uint SurveyGroupCount;
RWStructuredBuffer<float4> SurveyOutput;


//------------------------------------------------------- LDS

groupshared float4 SharedColor[TILE_SIZE];


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, 1, 1)]
void MainCS(uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Color = 0.0;
	{
		uint LoadCount = (SurveyGroupCount + TILE_SIZE - 1) / TILE_SIZE;

		LOOP
		for (uint i = 0; i < LoadCount; i++)
		{
			float4 LoadedColor = SurveyOutput[GroupThreadIndex + i * TILE_SIZE];

			if ((GroupThreadIndex + i * TILE_SIZE) >= SurveyGroupCount)
			{
				LoadedColor = 0.0;
			}

			if (SurveyReduceOp == 0)
			{
				Color = max(Color, LoadedColor);
			}
			else
			{
				Color += LoadedColor;
			}
		}
	}

	SharedColor[GroupThreadIndex] = Color;

	GroupMemoryBarrierWithGroupSync();
	
	// Reduction
	{
		UNROLL
		for (uint i = 32; i > 1; i /= 2)
		{
			
			if (SurveyReduceOp == 0)
			{
				Color = max(Color, SharedColor[GroupThreadIndex ^ i]);
			}
			else
			{
				Color += SharedColor[GroupThreadIndex ^ i];
			}
			
			GroupMemoryBarrierWithGroupSync();

			SharedColor[GroupThreadIndex] = Color;
			
			GroupMemoryBarrierWithGroupSync();
		}
		
		if (SurveyReduceOp == 0)
		{
			Color = max(Color, SharedColor[GroupThreadIndex ^ 0x1]);
		}
		else
		{
			Color += SharedColor[GroupThreadIndex ^ 0x1];
		}
	}

	BRANCH
	if (GroupThreadIndex == 0)
	{
		SurveyOutput[0] = Color;
	}
}
