// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

uint2 DstExtent;
uint2 ImageExtent;
float2 KernelSpatialTextureInvSize;
uint2 DstBufferExtent;
float KernelSupportScale;

StructuredBuffer<FBloomKernelInfo> KernelConstantsBuffer;

Texture2D SrcTexture;
SamplerState SrcSampler;

RWTexture2D<float4> DstTexture;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	FBloomKernelInfo OriginalKernelInfo = KernelConstantsBuffer[0];

	// f(x) = T(x)  0 < x < k
    //      = 0    x > k
	// g(x) = f(L * frac(x/L))   - periodic extention of f with period L
	// g_c(x) = f(L * frac( (x+c)/ L)  - periodic extension of offset f with period L
	//
	// 
	// The pixel that this thread 'owns'

	uint2 PixelId = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);
	
	// If the actual scan line length was not divisible by THREADS_PER_GROUP we will have a few extra threads that don't
	// own any pixels.  This will be 'false' for most thread groups.

	if (! (PixelId.x  < DstBufferExtent.x )) return;
	if (! (PixelId.y  < DstBufferExtent.y )) return;

	float2 DstSize = float2(DstExtent.x, DstExtent.y);

	// Where to find the center of the kernel, and the relative size of the kernel texture wrt the dst buffer.
	//float2 UVCenter    = KernelCenter;
	float2 UVCenter    = (float2(OriginalKernelInfo.CenterPixelCoord) + 0.5) * KernelSpatialTextureInvSize;
	float  ResizeScale = KernelSupportScale;

	// The minor axis of the target buffer
	// NB: I should change this to the minor axis of the viewing window..  The target buffer is bigger and some power of two. 
		
	uint MajorAxisPixelCount = max(ImageExtent.y, ImageExtent.x);

	// The number of kernels that could fit along the minor axis = ResizeScale.

	// The length of the kernel is pixels in the Dst Buffer.  Here the kernel is assumed
	// to come from a square texture.
	
	float KernelSizeInDstPixels = max( float(MajorAxisPixelCount) * (ResizeScale), 1.f);
	
	// We have a simple frac trick for making the periodic image, but it fails
	// in the following case.  Then we just have to do it by hand.
	bool bKernelTooBigForFrac = !(float(DstBufferExtent.y) > UVCenter.y * KernelSizeInDstPixels);
	bKernelTooBigForFrac = bKernelTooBigForFrac || !(float(DstBufferExtent.x) > UVCenter.x * KernelSizeInDstPixels);


	float4 ResultColor = float4(0.f, 0.f, 0.f, 0.f);

	if ( bKernelTooBigForFrac )
	{

		// distance from each corner

		float2 UL = float2(PixelId.x, PixelId.y) / KernelSizeInDstPixels;
		float2 UR = float2(DstBufferExtent.x - PixelId.x, PixelId.y) / KernelSizeInDstPixels;
		float2 LL = float2(PixelId.x, DstBufferExtent.y - PixelId.y) / KernelSizeInDstPixels;
		float2 LR = float2(DstBufferExtent.x - PixelId.x, DstBufferExtent.y - PixelId.y) / KernelSizeInDstPixels;

		float4 TmpColor = float4(0.f, 0.f ,0.f ,0.f);
		if (UL.x < 0.5 && UL.y < 0.5)
		{
			float4 SampledColor = SrcTexture.SampleLevel(SrcSampler, UL + UVCenter, 0);
			TmpColor.rgb += SampledColor.rgb;
			TmpColor.a = max(TmpColor.a, SampledColor.a);
		}
		if (UR.x < 0.5 && UR.y < 0.5)
		{
			float2 SamplePoint = float2(UVCenter.x - UR.x, UVCenter.y + UR.y);
			float4 SampledColor = SrcTexture.SampleLevel(SrcSampler,  SamplePoint, 0);
			TmpColor.rgb += SampledColor.rgb;
			TmpColor.a = max(TmpColor.a, SampledColor.a);
		}
		if (LL.x < 0.5 && LL.y < 0.5)
		{
			float2 SamplePoint = float2(UVCenter.x + LL.x, UVCenter.y - LL.y);
			float4 SampledColor = SrcTexture.SampleLevel(SrcSampler,  SamplePoint, 0);
			TmpColor.rgb += SampledColor.rgb;
			TmpColor.a = max(TmpColor.a, SampledColor.a);
		}
		if (LR.x < 0.5 && LR.y < 0.5)
		{
			float2 SamplePoint = float2(UVCenter.x - LR.x, UVCenter.y - LR.y);
			float4 SampledColor = SrcTexture.SampleLevel(SrcSampler,  SamplePoint, 0);
			TmpColor.rgb += SampledColor.rgb;
			TmpColor.a = max(TmpColor.a, SampledColor.a);
		}
		ResultColor = TmpColor;
	}
	else 
	{
		// Current Pixel in KernelUV space

		float2 PixelInKernelUV = float2(float(PixelId.x) / KernelSizeInDstPixels, float(PixelId.y) / KernelSizeInDstPixels);

		// The size of the Dst Buffer in terms of KernelSizeInDstPixels.  Note the Dst buffer is bigger than the image extent.
	
		float2 DstSizeInKernelUnits = float2( DstSize.x / KernelSizeInDstPixels, DstSize.y / KernelSizeInDstPixels);
	
		float2 InvDstSizeInKernelUnits = float2(1.f / DstSizeInKernelUnits.x, 1.f / DstSizeInKernelUnits.y );

	
	
		// Offset by 1/2, 1/2 so we start in the middle of the kernel texture
	 
		float2 OffsetPixelInKernelUV = PixelInKernelUV + UVCenter;

		// make periodic with period DstSizeInKernelUV

		OffsetPixelInKernelUV *= InvDstSizeInKernelUnits;
		OffsetPixelInKernelUV = frac(OffsetPixelInKernelUV); 
		OffsetPixelInKernelUV *= DstSizeInKernelUnits; 

		

		if (OffsetPixelInKernelUV.x < 1.f && OffsetPixelInKernelUV.y < 1.f)
		{
			float4 SampledColor = SrcTexture.SampleLevel(SrcSampler, OffsetPixelInKernelUV, 0);
			ResultColor.rgba += SampledColor; 
		}
	}

	// Force the alpha kernel to be a delta function at the pixel center.
#if 0	
	ResultColor.w = (PixelId.x == 0 && PixelId.y == 0) ? 1.f : 0.f;
	
#endif

	DstTexture[PixelId] =  ResultColor;

}
