// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONSTANTS

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

uint PassId;
uint2 ScatterDispersionTextureSize;
Texture2D ScatterDispersionTexture;
StructuredBuffer<float4> MaxScatterDispersionBuffer;

RWTexture2D<float4> ScatterDispersionOutput;


//------------------------------------------------------- LDS

groupshared float4 SharedColor[TILE_SIZE * TILE_SIZE];


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 MaxScatterDispersion = MaxScatterDispersionBuffer[0];
	
	uint2 InputPixelPos = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);

	// Init LDS
	float4 TexelColor = ScatterDispersionTexture[InputPixelPos];
	{
		bool bValidPixel = all(InputPixelPos < ScatterDispersionTextureSize);

		if (PassId == 0)
		{
			// Clamp the kernel to highest value found near the kernel center.
			TexelColor = clamp(TexelColor, 0.0, MaxScatterDispersion);
		}

		if (!bValidPixel)
		{
			TexelColor = 0.0;
		}

		SharedColor[GroupThreadIndex] = TexelColor;
	}

	GroupMemoryBarrierWithGroupSync();

	// Reduction
	{
		UNROLL
		for (uint i = 32; i > 1; i /= 2)
		{
			TexelColor += SharedColor[GroupThreadIndex ^ i];

			GroupMemoryBarrierWithGroupSync();

			SharedColor[GroupThreadIndex] = TexelColor;
			
			GroupMemoryBarrierWithGroupSync();
		}
		
		TexelColor += SharedColor[GroupThreadIndex ^ 0x1];
	}

	// Output
	if (GroupThreadIndex == 0)
	{
		ScatterDispersionOutput[GroupId] = TexelColor;
	}
}
