// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

float ViewTexelRadiusInKernelTexels;
uint SurveyGroupGridSize;


uint2 KernelSpatialTextureSize;
Texture2D KernelSpatialTexture;
StructuredBuffer<uint> KernelCenterCoordBuffer;
StructuredBuffer<float4> MaxScatterDispersionBuffer;

RWStructuredBuffer<float4> SurveyOutput;
RWTexture2D<float4> DebugOutput;


//------------------------------------------------------- LDS

groupshared float4 SharedColor[TILE_SIZE * TILE_SIZE];


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Debug = 0.0;

	uint LinearGroupId = dot(GroupId, uint2(1, SurveyGroupGridSize));
	int2 PixelOffset = int2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE) + TILE_SIZE * (GroupId - SurveyGroupGridSize / 2);

	// Write LDS
	float4 TexelColor;
	{
		uint2 KernelCenterPos = uint2(KernelCenterCoordBuffer[0], KernelCenterCoordBuffer[1]);
		
		int2 PixelOffset = int2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE) + TILE_SIZE * (GroupId - SurveyGroupGridSize / 2);

		uint2 InputPixelPos = (KernelCenterPos + PixelOffset) % KernelSpatialTextureSize;

		float IsSurveyZone = saturate(ViewTexelRadiusInKernelTexels + 0.5 - max(abs(PixelOffset.x), abs(PixelOffset.y)));
		//Debug.r = IsSurveyZone;

		TexelColor = max(KernelSpatialTexture[InputPixelPos] - MaxScatterDispersionBuffer[0], 0.0) * IsSurveyZone;
		//Debug = TexelColor;

		SharedColor[GroupThreadIndex] = TexelColor;
	}

	GroupMemoryBarrierWithGroupSync();

	// Reduction
	{
		UNROLL
		for (uint i = 32; i > 1; i /= 2)
		{
			TexelColor += SharedColor[GroupThreadIndex ^ i];

			GroupMemoryBarrierWithGroupSync();

			SharedColor[GroupThreadIndex] = TexelColor;
			
			GroupMemoryBarrierWithGroupSync();
		}
		
		TexelColor += SharedColor[GroupThreadIndex ^ 0x1];
	}

	BRANCH
	if (GroupThreadIndex == 0)
	{
		SurveyOutput[LinearGroupId] = TexelColor;
		Debug = TexelColor;
	}
	
	//DebugOutput[PixelOffset + TILE_SIZE * (SurveyGroupGridSize / 2)] = Debug;
}
