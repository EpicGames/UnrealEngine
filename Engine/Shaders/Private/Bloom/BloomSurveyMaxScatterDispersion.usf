// Copyright Epic Games, Inc. All Rights Reserved.

#include "BloomCommon.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

float ViewTexelRadiusInKernelTexels;
uint SurveyGroupGridSize;
uint SurveyType;


uint2 KernelSpatialTextureSize;
Texture2D KernelSpatialTexture;
StructuredBuffer<uint> KernelCenterCoordBuffer;

RWStructuredBuffer<float4> SurveyOutput;
RWTexture2D<float4> DebugOutput;


//------------------------------------------------------- LDS
groupshared uint SharedMaxNeighborColor[4];


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Debug = 0.0;

	uint LinearGroupId = dot(GroupId, uint2(1, SurveyGroupGridSize));
	int2 PixelOffset = int2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE) + TILE_SIZE * (GroupId - SurveyGroupGridSize / 2);

	// Init LDS
	if (GroupThreadIndex == 0)
	{
		SharedMaxNeighborColor[0] = 0;
		SharedMaxNeighborColor[1] = 0;
		SharedMaxNeighborColor[2] = 0;
		SharedMaxNeighborColor[3] = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	{
		uint2 KernelCenterPos = uint2(KernelCenterCoordBuffer[0], KernelCenterCoordBuffer[1]);

		uint2 InputPixelPos = (KernelCenterPos + PixelOffset) % KernelSpatialTextureSize;
		
		//float IsCenterZone = saturate(ViewTexelRadiusInKernelTexels + 0.5 - max(abs(PixelOffset.x), abs(PixelOffset.y)));
		//float IsEdgeOrCenterZone = saturate(ViewTexelRadiusInKernelTexels + 3.5 - max(abs(PixelOffset.x), abs(PixelOffset.y)));
				
		float IsCenterZone = saturate(2.0 * ViewTexelRadiusInKernelTexels + 1.0 - max(abs(PixelOffset.x), abs(PixelOffset.y)));
		float IsEdgeOrCenterZone = saturate(2.0 * ViewTexelRadiusInKernelTexels + 4.0 - max(abs(PixelOffset.x), abs(PixelOffset.y)));

		float IsSurveyZone = saturate(1.0 - IsCenterZone) * IsEdgeOrCenterZone;
		//Debug.r = IsSurveyZone;

		float4 TexelColor = KernelSpatialTexture[InputPixelPos] * IsSurveyZone;
		//Debug = TexelColor;

		TexelColor.rgb = max3(TexelColor.r, TexelColor.g, TexelColor.b).xxx;

		InterlockedMax(SharedMaxNeighborColor[0], asuint(TexelColor.r));
		InterlockedMax(SharedMaxNeighborColor[1], asuint(TexelColor.g));
		InterlockedMax(SharedMaxNeighborColor[2], asuint(TexelColor.b));
		InterlockedMax(SharedMaxNeighborColor[3], asuint(TexelColor.a));
	}

	GroupMemoryBarrierWithGroupSync();
	
	if (GroupThreadIndex == 0)
	{
		float4 MaxNeighborColor = float4(
			asfloat(SharedMaxNeighborColor[0]),
			asfloat(SharedMaxNeighborColor[1]),
			asfloat(SharedMaxNeighborColor[2]),
			asfloat(SharedMaxNeighborColor[3]));

		SurveyOutput[LinearGroupId] = MaxNeighborColor;
		
		//Debug = MaxNeighborColor;
	}
	
	//DebugOutput[PixelOffset + TILE_SIZE * (SurveyGroupGridSize / 2)] = Debug;
}
