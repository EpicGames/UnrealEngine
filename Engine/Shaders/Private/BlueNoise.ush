// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueNoiseCommon.ush"

/**
 * Spatiotemporal Blue Noise lookup texture with scalar output.  
 * Returns a uniformly distributed random variable in [0, 1] that is blue (no low frequencies) in both xy (screen space) and in z (frame index), which is designed to work with temporal accumulation.
 * The texture is 128x128x64 and tiled, based on "Spatiotemporal Blue Noise Masks" [Wolfe et al 2022] and 
 * https://developer.nvidia.com/blog/rendering-in-real-time-with-spatiotemporal-blue-noise-textures-part-1/
 */
float BlueNoiseScalar(uint2 ScreenCoord, uint FrameIndex)
{
	return GetBlueNoiseScalar(BlueNoise.ScalarTexture, BlueNoise.Dimensions, BlueNoise.ModuloMasks, ScreenCoord, FrameIndex);
}

/**
 * Spatiotemporal Blue Noise lookup texture with float2 output.
 * Returns a uniformly distributed random vector in [0, 1] that is blue (no low frequencies) in both xy (screen space) and in z (frame index), which is designed to work with temporal accumulation.
 * The texture is 128x128x64 and tiled.
 */
float2 BlueNoiseVec2(uint2 ScreenCoord, uint FrameIndex)
{
	return GetBlueNoiseVec2(BlueNoise.Vec2Texture, BlueNoise.Dimensions, BlueNoise.ModuloMasks, ScreenCoord, FrameIndex);
}