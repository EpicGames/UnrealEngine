// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float GetBlueNoiseScalar(Texture2D<float> InTexture, int3 InDimensions, int3 InModuloMasks, uint2 InScreenCoord, uint InFrameIndex)
{
	uint3 WrappedCoordinate = uint3(InScreenCoord, InFrameIndex) & InModuloMasks;
	uint3 TextureCoordinate = uint3(WrappedCoordinate.x, WrappedCoordinate.z * InDimensions.y + WrappedCoordinate.y, 0);
	return InTexture.Load(TextureCoordinate, 0).x;
}

float2 GetBlueNoiseVec2(Texture2D<float2> InTexture, int3 InDimensions, int3 InModuloMasks, uint2 InScreenCoord, uint InFrameIndex)
{
	uint3 WrappedCoordinate = uint3(InScreenCoord, InFrameIndex) & InModuloMasks;
	uint3 TextureCoordinate = uint3(WrappedCoordinate.x, WrappedCoordinate.z * BlueNoise.Dimensions.y + WrappedCoordinate.y, 0);
	return InTexture.Load(TextureCoordinate, 0).xy;
}

float GetBlueNoiseDitheredCoverage(Texture2D<float> InTexture, int3 InDimensions, int3 InModuloMasks, uint2 InScreenCoord, uint InFrameIndex, float InCoverage)
{
	// New dithering based on Blue noise
	const float Noise2 = GetBlueNoiseScalar(InTexture, InDimensions, InModuloMasks, InScreenCoord, InFrameIndex);
	const float LegacyAdjustement = 0.83f; // This is required to get the same look and empty areas as the legacy one.
	float Dither = LegacyAdjustement * Noise2;
	return InCoverage + Dither - 0.5f;
}