// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "Quaternion.ush"

struct FBoneTransform
{
	FQuat  Rotation;
	float3 Location;
};

struct FBoneTransformWithScale
{
	FQuat  Rotation;
	float3 Location;
	float3 Scale;
};

struct FBoneTransformWithUniformScale
{
	FQuat  Rotation;
	float3 Location;
	float  Scale;
};

////

FBoneTransform MakeBoneTransform(FQuat Rotation, float3 Location)
{
	FBoneTransform Transform;
	Transform.Rotation = Rotation;
	Transform.Location = Location;
	return Transform;
}

FBoneTransform MakeBoneTransformIdentity()
{
	return MakeBoneTransform(
		QuatIdentity(),
		float3(0.0f, 0.0f, 0.0f)
	);
}

FBoneTransform MultiplyBoneTransform(FBoneTransform Atom1, FBoneTransform Atom2)
{
	FQuat Rotation	= QuatMultiply(Atom2.Rotation, Atom1.Rotation);
	float3 Location	= QuatRotateVector(Atom2.Rotation, Atom1.Location) + Atom2.Location;
	return MakeBoneTransform(Rotation, Location);
}

FBoneTransform InvertBoneTransform(FBoneTransform Transform)
{
	FQuat Rotation	=  QuatConjugate(Transform.Rotation);
	float3 Location	= -QuatRotateVector(Rotation, Transform.Location);
	return MakeBoneTransform(Rotation, Location);
}

FBoneTransform RemoveBoneTransformScale(FBoneTransformWithScale Transform)
{
	FBoneTransform Result;
	Result.Location = Transform.Location;
	Result.Rotation = QuatNormalize(Transform.Rotation);
	return Result;
}

FBoneTransform RemoveBoneTransformUniformScale(FBoneTransformWithUniformScale Transform)
{
	FBoneTransform Result;
	Result.Location = Transform.Location;
	Result.Rotation = QuatNormalize(Transform.Rotation);
	return Result;
}

float4x4 MatrixFromBoneTransform(FBoneTransform Transform)
{
	float3x3 RotationMatrix = QuatToMatrix(Transform.Rotation);

	float4x4 OutMatrix;
	OutMatrix[0] = float4(RotationMatrix[0], 0.0);
	OutMatrix[1] = float4(RotationMatrix[1], 0.0);
	OutMatrix[2] = float4(RotationMatrix[2], 0.0);
	OutMatrix[3] = float4(Transform.Location, 1.0);
	return OutMatrix;
}

FBoneTransform BlendBoneTransforms(FBoneTransform Atom1, FBoneTransform Atom2, float Alpha)
{
	FBoneTransform Blended;
	Blended.Location = lerp(Atom1.Location, Atom2.Location, Alpha);
	Blended.Rotation = QuatSlerpApproximate(Atom1.Rotation, Atom2.Rotation, Alpha);
	return Blended;
}

////

FBoneTransformWithScale MakeBoneTransformWithScale(FQuat Rotation, float3 Location, float3 Scale)
{
	FBoneTransformWithScale Transform;
	Transform.Rotation = Rotation;
	Transform.Location = Location;
	Transform.Scale = Scale;
	return Transform;
}

FBoneTransformWithScale MakeBoneTransformWithScaleIdentity()
{
	return MakeBoneTransformWithScale(QuatIdentity(), float3(0.0f, 0.0f, 0.0f), float3(1.0f, 1.0f, 1.0f));
}

FBoneTransformWithScale MultiplyBoneTransformWithScale(FBoneTransformWithScale Atom1, FBoneTransformWithScale Atom2)
{
	// NOTE: Assumes no negative scale! Otherwise should multiply using matrices
	FQuat Rotation	= QuatMultiply(Atom2.Rotation, Atom1.Rotation);
	float3 Location	= QuatRotateVector(Atom2.Rotation, Atom2.Scale * Atom1.Location) + Atom2.Location;
	float3 Scale = Atom1.Scale * Atom2.Scale;
	return MakeBoneTransformWithScale(Rotation, Location, Scale);
}

float4x4 MatrixFromBoneTransformWithScale(FBoneTransformWithScale Transform)
{
	float3x3 RotationMatrix = QuatToMatrix(Transform.Rotation);

	RotationMatrix[0] *= Transform.Scale.xxx;
	RotationMatrix[1] *= Transform.Scale.yyy;
	RotationMatrix[2] *= Transform.Scale.zzz;

	float4x4 OutMatrix;
	OutMatrix[0] = float4(RotationMatrix[0], 0.0);
	OutMatrix[1] = float4(RotationMatrix[1], 0.0);
	OutMatrix[2] = float4(RotationMatrix[2], 0.0);
	OutMatrix[3] = float4(Transform.Location, 1.0);
	return OutMatrix;
}

FBoneTransformWithScale BlendBoneTransformsWithScale(FBoneTransformWithScale Atom1, FBoneTransformWithScale Atom2, float Alpha)
{
	FBoneTransformWithScale Blended;
	Blended.Location = lerp(Atom1.Location, Atom2.Location, Alpha);
	Blended.Rotation = QuatSlerpApproximate(Atom1.Rotation, Atom2.Rotation, Alpha);
	Blended.Scale    = lerp(Atom1.Scale, Atom2.Scale, Alpha);
	return Blended;
}

////

FBoneTransformWithUniformScale MakeBoneTransformWithUniformScale(FQuat Rotation, float3 Location, float Scale)
{
	FBoneTransformWithUniformScale Transform;
	Transform.Rotation = Rotation;
	Transform.Location = Location;
	Transform.Scale = Scale;
	return Transform;
}

FBoneTransformWithUniformScale MakeBoneTransformWithUniformScaleIdentity()
{
	return MakeBoneTransformWithUniformScale(QuatIdentity(), float3(0.0f, 0.0f, 0.0f), 1.0);
}

FBoneTransformWithUniformScale MultiplyBoneTransformWithUniformScale(FBoneTransformWithUniformScale Atom1, FBoneTransformWithUniformScale Atom2)
{
	// NOTE: Assumes no negative scale! Otherwise should multiply using matrices
	FQuat Rotation	= QuatMultiply(Atom2.Rotation, Atom1.Rotation);
	float3 Location	= QuatRotateVector(Atom2.Rotation, Atom2.Scale * Atom1.Location) + Atom2.Location;
	float Scale = Atom1.Scale * Atom2.Scale;
	return MakeBoneTransformWithUniformScale(Rotation, Location, Scale);
}

float4x4 MatrixFromBoneTransformWithUniformScale(FBoneTransformWithUniformScale Transform)
{
	float3x3 RotationMatrix = QuatToMatrix(Transform.Rotation);

	RotationMatrix[0] *= Transform.Scale;
	RotationMatrix[1] *= Transform.Scale;
	RotationMatrix[2] *= Transform.Scale;

	float4x4 OutMatrix;
	OutMatrix[0] = float4(RotationMatrix[0], 0.0);
	OutMatrix[1] = float4(RotationMatrix[1], 0.0);
	OutMatrix[2] = float4(RotationMatrix[2], 0.0);
	OutMatrix[3] = float4(Transform.Location, 1.0);
	return OutMatrix;
}

FBoneTransformWithUniformScale BlendBoneTransformsWithUniformScale(FBoneTransformWithUniformScale Atom1, FBoneTransformWithUniformScale Atom2, float Alpha)
{
	FBoneTransformWithUniformScale Blended;
	Blended.Location = lerp(Atom1.Location, Atom2.Location, Alpha);
	Blended.Rotation = QuatSlerpApproximate(Atom1.Rotation, Atom2.Rotation, Alpha);
	Blended.Scale    = lerp(Atom1.Scale, Atom2.Scale, Alpha);
	return Blended;
}

FBoneTransformWithUniformScale ApplyAdditiveLocalSpace(FBoneTransformWithUniformScale Base, FBoneTransformWithUniformScale Additive, float Weight)
{
	FQuat AdditiveRotation = QuatSlerpApproximate(QuatIdentity(), Additive.Rotation, Weight);

	FBoneTransformWithUniformScale Result;
	Result.Rotation = QuatNormalize(QuatMultiply(AdditiveRotation, Base.Rotation));
	Result.Location = Base.Location + Weight * Additive.Location;
	Result.Scale    = Base.Scale * lerp(1.0, Additive.Scale, Weight);
	return Result;
}