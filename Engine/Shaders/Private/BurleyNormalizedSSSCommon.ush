// Copyright Epic Games, Inc. All Rights Reserved.

/*=========================================================================
	BurleyNormalizedSSSCommon.ush: Burley common functions.
=========================================================================*/
#pragma once

#include "SubsurfaceProfileCommon.ush"

inline float3 Burley_Profile(float Radius, float3 Albedo, float3 S3D, float L)
{   //R(r)r
	float3 D = 1 / S3D;
	float R = Radius / L;
	const float Inv8Pi = 1.0 / (8 * PI);
	float3 NegRbyD = -R / D;
	return Albedo * max((exp(NegRbyD) + exp(NegRbyD / 3.0)) / (D*L)*Inv8Pi, 0);
}

//Diffuse profile basic formula
//    D : shapes the height and width of the profile curve.
//	  Radius : the distance between the entering surface point and the exit surface point.
//Assume: r and d >0
float GetDiffuseReflectProfile(float D, float Radius)
{
	//The diffuse reflectance profile:
	//R(d,r) = \frac{e^{-r/d}+e^{-r/(3d)}}{8*pi*d*r}
	const float Inv8Pi = 1.0 / (8 * PI);
	float NegRbyD = -Radius / D;
	return (exp(NegRbyD) + exp(NegRbyD / 3.0)) / (D*Radius)*Inv8Pi;
}

float3 GetDiffuseReflectProfileWithDiffuseMeanFreePath(float3 L, float3 S3D, float Radius)
{
	//rR(r)
	float3 D = 1 / S3D;
	float3 R = Radius / L;
	const float Inv8Pi = 1.0 / (8 * PI);
	float3 NegRbyD = -R / D;
	return max((exp(NegRbyD) + exp(NegRbyD / 3.0)) / (D * L) * Inv8Pi, 0.000000000001f /*Fix color shift due to precision issue for substrate per-pixel MFP SSS*/);
}

float GetComponentForScalingFactorEstimation(float4 SurfaceAlbedo)
{
	return SurfaceAlbedo.a;
}

float GetDiffuseMeanFreePathForSampling(float4 DiffuseMeanFreePath)
{
	return DiffuseMeanFreePath.a;
}

//-------------------------------------------------------------------------
// Three scaling factor function
// Method 1: The light directly goes into the volume in a direction perpendicular to the surface.
// Average relative error: 5.5% (reference to MC)
float GetPerpendicularScalingFactor(float SurfaceAlbedo)
{
	// from the paper, the formula explicitly has an abs for (A-0.8)
	float Value = abs(SurfaceAlbedo - 0.8);
	return 1.85 - SurfaceAlbedo + 7 * Value * Value * Value;
}

float3 GetPerpendicularScalingFactor3D(float3 SurfaceAlbedo)
{
	float3 Value = abs(SurfaceAlbedo - 0.8);
	return 1.85 - SurfaceAlbedo + 7 * Value * Value * Value;
}

// Method 2: Ideal diffuse transmission at the surface. More appropriate for rough surface.
// Average relative error: 3.9% (reference to MC)
float GetDiffuseSurfaceScalingFactor(float SurfaceAlbedo)
{
	float Value = SurfaceAlbedo - 0.8;
	return 1.9 - SurfaceAlbedo + 3.5 * Value * Value;
}

float3 GetDiffuseSurfaceScalingFactor3D(float3 SurfaceAlbedo)
{
	float3 Value = SurfaceAlbedo - 0.8;
	return 1.9 - SurfaceAlbedo + 3.5 * Value * Value;
}

// Method 3: The spectral of diffuse mean free path on the surface.
// Avergate relative error: 7.7% (reference to MC)
float GetSearchLightDiffuseScalingFactor(float SurfaceAlbedo)
{
	float Value = SurfaceAlbedo - 0.33;
	return 3.5 + 100 * Value * Value * Value * Value;
}

float3 GetSearchLightDiffuseScalingFactor3D(float3 SurfaceAlbedo)
{
	float3 Value = SurfaceAlbedo - 0.33;
	return 3.5 + 100 * Value * Value * Value * Value;
}

// TODO: find the source of this magic number to convert dmfp to mfp to match path tracer and rasterizer.
// DMFP/SearchLightDiffuseScalingFactor goes to the normalized distance(d=l/s), then multiply the PerpendicularScalingFactor to get the mfp
// float Dmfp2MfpMagicNumber = 0.6f;

// Mathmatically matching based on diffusion coefficient instead of burley's approximation. However, it leads to incorrect result as 
// we use burley's approximation (IOR=1.0) for screenspace diffuse scattering.
//float3 Alpha = 1 - exp(-11.43 * SurfaceAlbedo + 15.38 * SurfaceAlbedo * SurfaceAlbedo - 13.91 * SurfaceAlbedo * SurfaceAlbedo * SurfaceAlbedo);
//return DMFP * sqrt(3 * (1 - Alpha) / (2 - Alpha));
float3 GetMFPFromDMFPCoeff(float3 DMFPSurfaceAlbedo, float3 MFPSurfaceAlbedo, float Dmfp2MfpMagicNumber = 0.6f)
{
	return Dmfp2MfpMagicNumber * GetPerpendicularScalingFactor3D(MFPSurfaceAlbedo) / GetSearchLightDiffuseScalingFactor3D(DMFPSurfaceAlbedo);
}

float3 GetMFPFromDMFPApprox(float3 SurfaceAlbedo, float3 TargetSurfaceAlbedo, float3 DMFP)
{
	return GetMFPFromDMFPCoeff(SurfaceAlbedo, TargetSurfaceAlbedo) * DMFP;
}

float3 GetDMFPFromMFPApprox(float3 SurfaceAlbedo, float3 MFP)
{
	float3 MFPFromDMFPCoeff = GetMFPFromDMFPCoeff(SurfaceAlbedo, SurfaceAlbedo);
	return MFP / MFPFromDMFPCoeff;
}

// With world unit scale 
float4 GetSubsurfaceProfileMFPInCm(int SubsurfaceProfileInt)
{
	float4 DMFP = GetSubsurfaceProfileDMFPInCm(SubsurfaceProfileInt);
	float4 SurfaceAlbedo = GetSubsurfaceProfileSurfaceAlbedo(SubsurfaceProfileInt);

	return float4(GetMFPFromDMFPApprox(SurfaceAlbedo.xyz, SurfaceAlbedo.xyz, DMFP.xyz),0.0f);
}

float GetScalingFactor(float A)
{
#if LIGHT_PERPENDICULAR
	float S = GetPerpendicularScalingFactor(A);
#elif (LIGHT_DIFFUSESURFACE)
	float S = GetDiffuseSurfaceScalingFactor(A);
#elif (LIGHT_PERPENDICULAR_DIFFUSE_SURFACE)
	float S = GetSearchLightDiffuseScalingFactor(A);
#endif
	return S;
}

float3 GetScalingFactor3D(float3 SurfaceAlbedo)
{
#if LIGHT_PERPENDICULAR
	float3 S3D = GetPerpendicularScalingFactor3D(SurfaceAlbedo);
#elif (LIGHT_DIFFUSESURFACE)
	float3 S3D = GetDiffuseSurfaceScalingFactor3D(SurfaceAlbedo);
#elif (LIGHT_PERPENDICULAR_DIFFUSE_SURFACE)
	float3 S3D = GetSearchLightDiffuseScalingFactor3D(SurfaceAlbedo);
#endif
	return S3D;
}

float3 GetCDF3D(float3 D, float X)
{
	return 1 - 0.25 * exp(-X / D) - 0.75 * exp(-X / (3 * D));
}

float GetCDF(float D, float X, float XI)
{
	return 1 - 0.25*exp(-X / D) - 0.75*exp(-X / (3 * D)) - XI;
}
float GetCDFDeriv1(float D, float X)
{
	return 0.25 / D * (exp(-X / D) + exp(-X / (3 * D)));
}
float GetCDFDeriv1InversD(float InvD, float X)
{
	return 0.25 * InvD * (exp(-X*InvD)+exp(-3*X*InvD));
}

float GetCDFDeriv2(float D, float X)
{
	return exp(-X / D)*(-0.0833333*exp(2 * X / (3 * D)) - 0.25) / (D*D);
}

float RadiusRootFindAnalytic(float D, float RandomNumber)
{
	// solve for:
	//     F(X) = .25*exp(-X) + .75*exp(-X/3)
	//     cdf(X) = 1 - F(X/D)
	// so to find X s.t.
	//     cdf(X) = RandomNumber
	// we need to solve:
	//     1 - RandomNumber = F(X/D)
	//     Finv(1 - RandomNumber) * D = X
	// Finv(X) actually has an analytic solution:
	//     G = 1 + 4*U*(2*U + sqrt(1+4*U*U));
	//     Inv3 = 1.0 / 3.0;
	//     X/D = 3 * log((1+pow(G,-Inv3)+pow(G,Inv3))/(4*U));

	float U = 1 - RandomNumber;
	float G = 1.0f + 4*U*(2*U + sqrt(1+4*U*U));
	float Inv3 = 1.0f / 3.0f;
	float X = D * (3 * log( (1+pow(G,-Inv3)+pow(G,Inv3))/(4*U)));
	return X;
}

// Without clamp, the result will be zero which will lead to infinity for R/pdf. and Nan for the weighted sum.
// We need to clamp pdf to a very small value. this clamp is based on value below
#define CLAMP_PDF 0.00001

// Given D and a random number, use root finding method to calculate the radius
// in meters
float RadiusRootFinding(float D, float RandomNumber, float X0)
{
	// Make the distribution correct.
	// r=cdf^(-1)(p)     if p<1-CLAMP_PDF
	//  =cdf^(-1)(1-ClAMP_PDF) P>=1-CLAMP_PDF
	// RandomNumber = clamp(RandomNumber,0,1-CLAMP_PDF);

	const int NumOfIteration = 10;
	float Xn = X0;

	UNROLL for (int i = 0; i < NumOfIteration; ++i)
	{
		float Fx = GetCDF(D, Xn, RandomNumber);
		float DFx = GetCDFDeriv1(D, Xn);
		float DFxx = GetCDFDeriv2(D, Xn);
		Xn = Xn - 2 * Fx*DFx / (2 * DFx*DFx - Fx * DFxx);
	}

	return Xn;
}

// Brian's approximation.
float RadiusRootFindByApproximation(float D, float RandomNumber)
{
	return D * ((2 - 2.6)*RandomNumber - 2)*log(1 - RandomNumber);
}

// Get the probability to sample a disk.
float GetPdf(float Radius, float L, float S)
{
	//without clamp, the result will be zero which will lead to infinity for R/pdf. and Nan for the weighted sum.
	//we need to clamp this to a very small pdf.

	float Pdf = GetCDFDeriv1(L / S, Radius);
	return  max(Pdf, CLAMP_PDF);
}

float GetPdfInvD(float Radius, float InvD)
{
	float Pdf = GetCDFDeriv1InversD(InvD,Radius);
	return max(Pdf, CLAMP_PDF);
}

//@TODO Revisit here after offline comparison.
//call this function in TransmissionCommon.ush for Debug. No transmission tint is applied.

// This code is not used
#if 0
float4 GetBurleyTransmissionProfile(FGBufferData GBufferData, float Thickness)
{
	// 0..255, which SubSurface profile to pick

	uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(GBufferData);
	float WorldUnitScale = DecodeWorldUnitScale(View.SSProfilesTexture.Load(int3(SSSS_TINT_SCALE_OFFSET, SubsurfaceProfileInt,0)).a)*10.0f; //in cm. we do not multiply by 100.0f as we did in the subsurface shader.
	float4 MeanFreePath = DecodeDiffuseMeanFreePath(View.SSProfilesTexture.Load(int3(BSSS_DMFP_OFFSET, SubsurfaceProfileInt, 0)));
	float3 SurfaceAlbedo = View.SSProfilesTexture.Load(int3(BSSS_SURFACEALBEDO_OFFSET, SubsurfaceProfileInt, 0)).rgb;
	float3 ScalingFactor = GetSearchLightDiffuseScalingFactor3D(SurfaceAlbedo);//assuming that the volume albedo is the same to the surface albedo for transmission;
	float4 Output=float4(0,0,0,1);
	
	float3 r = Thickness / MeanFreePath.xyz;
	Output.xyz= 0.25*SurfaceAlbedo*(exp(-ScalingFactor * r) + 3 * exp(-ScalingFactor * r / 3));
	return Output;
}
#endif

float3 InternalGetBurleyTransmissionProfile(
	float3 SubsurfaceAlebdo, 
	float3 MeanFreePathInCm,
	float ThicknessInCm)
{
	// Assuming that the volume albedo is the same to the surface albedo for transmission;
	const float3 ScalingFactor = GetPerpendicularScalingFactor3D(SubsurfaceAlebdo);
	const float3 r = ThicknessInCm / MeanFreePathInCm.xyz;
	const float3 Output = 0.25 * SubsurfaceAlebdo * (exp(-ScalingFactor * r) + 3 * exp(-ScalingFactor * r / 3));

	return Output;
}

float3 GetBurleyTransmission(
	float3 SubsurfaceAlebdo,
	float3 MeanFreePathInCm,
	float ThicknessInCm)
{
	const float TransmissionMFPScaleFactor = 100.f; // This factor compensate a mistake in SSS profile transmission computation. See BurleyNormalizedSSS.cpp for more details
	return InternalGetBurleyTransmissionProfile(SubsurfaceAlebdo, MeanFreePathInCm * TransmissionMFPScaleFactor, ThicknessInCm);
}

float3 GetBurleyTransmissionProfile(
	float3 SubsurfaceAlebdo,
	float3 MeanFreePathInCm,
	float ThicknessInCm)
{
	const float  TransmissionMFPScaleFactor = 100.f; // This factor compensate a mistake in SSS profile transmission computation. See BurleyNormalizedSSS.cpp for more details
	const float3 TransmissionThroughput = InternalGetBurleyTransmissionProfile(SubsurfaceAlebdo, MeanFreePathInCm * TransmissionMFPScaleFactor, ThicknessInCm).xyz;

	// Replicate SSS profile distance fade out, by fading the tail of the curve to black.
	//  * SSS Profile stores transmittance into a discrete set of samples: BSSS_TRANSMISSION_PROFILE_SIZE(=32)
	//  * The transmission distance is set to a max of SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE(=5) (in scaled cm, depend on the WorldToUnitScale, cm by default)
	//
	// This code fades the last value to black.
	//  FadeDistance = Distance * SampleCount - MaxDistance * (SampleCount-0.5f)
	//  FadeDistance = Distance * 32.f - 5.f * 31.5f
	//
	// Transmission
	// 1 ___________
	//				\
	// 0             \ _______
	// ----------------------->
	// .. 29  30  31  32  Sample Index / Distance
	const float FadeDistance = 1.0f - saturate(ThicknessInCm * BSSS_TRANSMISSION_PROFILE_SIZE - SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE * (BSSS_TRANSMISSION_PROFILE_SIZE - 0.5f));

	return TransmissionThroughput * FadeDistance;
}

struct FBurleyParameter
{
	float4 SurfaceAlbedo;
	float4 DiffuseMeanFreePath;
	float  WorldUnitScale;
	float  SurfaceOpacity; // 1.0 means full Burley, 0.0 means full Default Lit
};