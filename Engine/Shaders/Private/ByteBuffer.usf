// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ComputeShaderUtils.ush"

#define RESOURCE_TYPE_FLOAT4_BUFFER				(0)
#define RESOURCE_TYPE_STRUCTURED_BUFFER			(1)
#define RESOURCE_TYPE_UINT_BUFFER				(2)
#define RESOURCE_TYPE_UINT4_ALIGNED_BUFFER		(3)

#define STRUCTURED_ELEMENT_SIZE_UINT1 (0)
#define STRUCTURED_ELEMENT_SIZE_UINT2 (1)
#define STRUCTURED_ELEMENT_SIZE_UINT4 (2)
#define STRUCTURED_ELEMENT_SIZE_UINT8 (3)

uint Value;
uint Size;
uint NumScatters;
uint SrcOffset;
uint DstOffset;

struct FUint8
{
	uint Value0;
	uint Value1;
	uint Value2;
	uint Value3;
	uint Value4;
	uint Value5;
	uint Value6;
	uint Value7;
};

#if STRUCTURED_ELEMENT_SIZE_UINT1 == STRUCTURED_ELEMENT_SIZE
	#define STRUCTURED_BUFFER_ACCESS(BufferName, Index) BufferName##1x[(Index)]
	#define STRUCTURED_VALUE Value
#elif STRUCTURED_ELEMENT_SIZE_UINT2 == STRUCTURED_ELEMENT_SIZE
	#define STRUCTURED_BUFFER_ACCESS(BufferName, Index) BufferName##2x[(Index)]
	#define STRUCTURED_VALUE uint2(Value, Value)
#elif STRUCTURED_ELEMENT_SIZE_UINT4 == STRUCTURED_ELEMENT_SIZE
	#define STRUCTURED_BUFFER_ACCESS(BufferName, Index) BufferName##4x[(Index)]
	#define STRUCTURED_VALUE uint4(Value,Value,Value,Value)
#elif STRUCTURED_ELEMENT_SIZE_UINT8 == STRUCTURED_ELEMENT_SIZE
	#define STRUCTURED_BUFFER_ACCESS(BufferName, Index) BufferName##8x[(Index)]
	FUint8 MakeUint8(uint InValue)
	{
		const FUint8 Uint8Value = {InValue,InValue,InValue,InValue,InValue,InValue,InValue,InValue };
		return Uint8Value;
	}
	#define STRUCTURED_VALUE MakeUint8(Value)
#endif

#if RESOURCE_TYPE == RESOURCE_TYPE_FLOAT4_BUFFER
	StructuredBuffer<uint>			ScatterStructuredBuffer;
	Buffer<float4>					SrcBuffer;
	StructuredBuffer<float4>		UploadStructuredBuffer4x;
	RWBuffer<float4>				DstBuffer;
#elif RESOURCE_TYPE == RESOURCE_TYPE_STRUCTURED_BUFFER

	StructuredBuffer<uint>			ScatterStructuredBuffer;

	StructuredBuffer<uint>		UploadStructuredBuffer1x;
	StructuredBuffer<uint2>		UploadStructuredBuffer2x;
	StructuredBuffer<uint4>		UploadStructuredBuffer4x;
	StructuredBuffer<FUint8>		UploadStructuredBuffer8x;

	StructuredBuffer<uint>		SrcStructuredBuffer1x;
	StructuredBuffer<uint2>		SrcStructuredBuffer2x;
	StructuredBuffer<uint4>		SrcStructuredBuffer4x;
	StructuredBuffer<FUint8>		SrcStructuredBuffer8x;

	RWStructuredBuffer<uint>	DstStructuredBuffer1x;
	RWStructuredBuffer<uint2>	DstStructuredBuffer2x;
	RWStructuredBuffer<uint4>	DstStructuredBuffer4x;
	RWStructuredBuffer<FUint8>	DstStructuredBuffer8x;
	
#elif RESOURCE_TYPE == RESOURCE_TYPE_UINT_BUFFER || RESOURCE_TYPE == RESOURCE_TYPE_UINT4_ALIGNED_BUFFER
	ByteAddressBuffer				ScatterByteAddressBuffer;
	ByteAddressBuffer				SrcByteAddressBuffer;
	ByteAddressBuffer				UploadByteAddressBuffer;
	RWByteAddressBuffer				DstByteAddressBuffer;
#endif

[numthreads(64, 1, 1)]
void MemsetBufferCS( uint3 GroupID : SV_GroupID, uint GroupIndex : SV_GroupIndex )
{
	const uint ThreadId = GetUnWrappedDispatchThreadId(GroupID, GroupIndex, 64);

#if RESOURCE_TYPE == RESOURCE_TYPE_FLOAT4_BUFFER
	// Size is in float4s
	if (ThreadId < Size)
	{
		DstBuffer[DstOffset + ThreadId] = asfloat(Value);
	}
#elif RESOURCE_TYPE == RESOURCE_TYPE_STRUCTURED_BUFFER
	// Size is in float4s
	if( ThreadId < Size )
	{
		STRUCTURED_BUFFER_ACCESS(DstStructuredBuffer, DstOffset + ThreadId) = STRUCTURED_VALUE;
	}
#elif RESOURCE_TYPE == RESOURCE_TYPE_UINT_BUFFER || RESOURCE_TYPE == RESOURCE_TYPE_UINT4_ALIGNED_BUFFER
	// Size and offset is in dwords
	uint DstIndex = DstOffset + ThreadId * 4;

	if( ThreadId * 4 + 3 < Size )
	{
		DstByteAddressBuffer.Store4( DstIndex * 4, uint4( Value, Value, Value, Value ) );
	}
	else if( ThreadId * 4 + 2 < Size )
	{
		DstByteAddressBuffer.Store3( DstIndex * 4, uint3( Value, Value, Value ) );
	}
	else if( ThreadId * 4 + 1 < Size )
	{
		DstByteAddressBuffer.Store2( DstIndex * 4, uint2( Value, Value ) );
	}
	else if( ThreadId * 4 < Size )
	{
		DstByteAddressBuffer.Store( DstIndex * 4, Value );
	}
#else
	#error "Not implemented"
#endif
}

[numthreads(64, 1, 1)]
void MemcpyCS( uint3 GroupID : SV_GroupID, uint GroupIndex : SV_GroupIndex )
{
	const uint ThreadId = GetUnWrappedDispatchThreadId(GroupID, GroupIndex, 64);

#if RESOURCE_TYPE == RESOURCE_TYPE_FLOAT4_BUFFER
	// Size is in float4s
	if (ThreadId < Size)
	{
		DstBuffer[DstOffset + ThreadId] = SrcBuffer[SrcOffset + ThreadId];
	}
#elif RESOURCE_TYPE == RESOURCE_TYPE_STRUCTURED_BUFFER
	// Size is in float4s
	if( ThreadId < Size )
	{
		STRUCTURED_BUFFER_ACCESS(DstStructuredBuffer, DstOffset + ThreadId) = STRUCTURED_BUFFER_ACCESS(SrcStructuredBuffer, SrcOffset + ThreadId);
	}
#elif RESOURCE_TYPE == RESOURCE_TYPE_UINT_BUFFER || RESOURCE_TYPE == RESOURCE_TYPE_UINT4_ALIGNED_BUFFER
	// Size and offsets are in dwords
	uint SrcIndex = SrcOffset + ThreadId * 4;
	uint DstIndex = DstOffset + ThreadId * 4;

	if( ThreadId * 4 + 3 < Size )
	{
		uint4 SrcData = SrcByteAddressBuffer.Load4( SrcIndex * 4 );
		DstByteAddressBuffer.Store4( DstIndex * 4, SrcData );
	}
	else if( ThreadId * 4 + 2 < Size )
	{
		uint3 SrcData = SrcByteAddressBuffer.Load3( SrcIndex * 4 );
		DstByteAddressBuffer.Store3( DstIndex * 4, SrcData );
	}
	else if( ThreadId * 4 + 1 < Size )
	{
		uint2 SrcData = SrcByteAddressBuffer.Load2( SrcIndex * 4 );
		DstByteAddressBuffer.Store2( DstIndex * 4, SrcData );
	}
	else if( ThreadId * 4 < Size )
	{
		uint SrcData = SrcByteAddressBuffer.Load( SrcIndex * 4 );
		DstByteAddressBuffer.Store( DstIndex * 4, SrcData );
	}
#else
	#error "Not implemented"
#endif
}

[numthreads(64, 1, 1)]
void ScatterCopyCS( uint3 GroupID : SV_GroupID, uint GroupIndex : SV_GroupIndex )
{
	const uint ThreadId = GetUnWrappedDispatchThreadId(GroupID, GroupIndex, 64) + SrcOffset;

	uint ScatterIndex = ThreadId / Size;
	uint ScatterOffset = ThreadId - ScatterIndex * Size;

	if( ScatterIndex < NumScatters )
	{
#if RESOURCE_TYPE == RESOURCE_TYPE_FLOAT4_BUFFER
		uint DstIndex = ScatterStructuredBuffer[ScatterIndex] * Size + ScatterOffset;
		uint SrcIndex = ThreadId;
		DstBuffer[DstIndex] = UploadStructuredBuffer4x[SrcIndex];
#elif RESOURCE_TYPE == RESOURCE_TYPE_STRUCTURED_BUFFER
		uint DstIndex = ScatterStructuredBuffer[ ScatterIndex ] * Size + ScatterOffset;
		uint SrcIndex = ThreadId;
		STRUCTURED_BUFFER_ACCESS(DstStructuredBuffer, DstIndex) = STRUCTURED_BUFFER_ACCESS(UploadStructuredBuffer, SrcIndex);
#elif RESOURCE_TYPE == RESOURCE_TYPE_UINT4_ALIGNED_BUFFER
		uint DstIndex = ScatterByteAddressBuffer.Load( ScatterIndex * 4 ) * Size + ScatterOffset;
		uint SrcIndex = ThreadId;
		uint4 SrcData = UploadByteAddressBuffer.Load4( SrcIndex * 16 );
		DstByteAddressBuffer.Store4( DstIndex * 16, SrcData );
#elif RESOURCE_TYPE == RESOURCE_TYPE_UINT_BUFFER
		uint DstIndex = ScatterByteAddressBuffer.Load( ScatterIndex * 4 ) * Size + ScatterOffset;
		uint SrcIndex = ThreadId;
		uint SrcData = UploadByteAddressBuffer.Load( SrcIndex * 4 );
		DstByteAddressBuffer.Store(DstIndex * 4, SrcData);
#else
	#error "Not implemented"
#endif
	}
}
