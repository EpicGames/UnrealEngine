// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

void LoadAndIncrementOffset(inout uint Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	Value = InBuffer.Load(Offset);
	Offset += 4;
}

void LoadAndIncrementOffset(inout uint2 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	Value = InBuffer.Load2(Offset);
	Offset += 8;
}

void LoadAndIncrementOffset(inout uint3 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	Value = InBuffer.Load3(Offset);
	Offset += 12;
}

void LoadAndIncrementOffset(inout uint4 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	Value = InBuffer.Load4(Offset);
	Offset += 16;
}

void LoadAndIncrementOffset(inout float Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	uint UintValue;
	LoadAndIncrementOffset(UintValue, InBuffer, Offset);
	Value = asfloat(UintValue);
}

void LoadAndIncrementOffset(inout float2 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	uint2 UintValue;
	LoadAndIncrementOffset(UintValue, InBuffer, Offset);
	Value = asfloat(UintValue);
}

void LoadAndIncrementOffset(inout float3 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	uint3 UintValue;
	LoadAndIncrementOffset(UintValue, InBuffer, Offset);
	Value = asfloat(UintValue);
}

void LoadAndIncrementOffset(inout float4 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{
	uint4 UintValue;
	LoadAndIncrementOffset(UintValue, InBuffer, Offset);
	Value = asfloat(UintValue);
}

void LoadAndIncrementOffset(inout float4x4 Value, ByteAddressBuffer InBuffer, inout uint Offset)
{	
	LoadAndIncrementOffset(Value[0], InBuffer, Offset);
	LoadAndIncrementOffset(Value[1], InBuffer, Offset);
	LoadAndIncrementOffset(Value[2], InBuffer, Offset);
	LoadAndIncrementOffset(Value[3], InBuffer, Offset);
}

void StoreAndIncrementOffset(uint Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	OutBuffer.Store(Offset, Value);
	Offset += 4;
}

void StoreAndIncrementOffset(uint2 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	OutBuffer.Store2(Offset, Value);
	Offset += 8;
}

void StoreAndIncrementOffset(uint3 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	OutBuffer.Store3(Offset, Value);
	Offset += 12;
}

void StoreAndIncrementOffset(uint4 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	OutBuffer.Store4(Offset, Value);
	Offset += 16;
}

void StoreAndIncrementOffset(float Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	StoreAndIncrementOffset(asuint(Value), OutBuffer, Offset);
}

void StoreAndIncrementOffset(float2 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	StoreAndIncrementOffset(asuint(Value), OutBuffer, Offset);
}

void StoreAndIncrementOffset(float3 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	StoreAndIncrementOffset(asuint(Value), OutBuffer, Offset);
}

void StoreAndIncrementOffset(float4 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{
	StoreAndIncrementOffset(asuint(Value), OutBuffer, Offset);
}

void StoreAndIncrementOffset(float4x4 Value, RWByteAddressBuffer OutBuffer, inout uint Offset)
{	
	StoreAndIncrementOffset(Value[0], OutBuffer, Offset);
	StoreAndIncrementOffset(Value[1], OutBuffer, Offset);
	StoreAndIncrementOffset(Value[2], OutBuffer, Offset);
	StoreAndIncrementOffset(Value[3], OutBuffer, Offset);
}
