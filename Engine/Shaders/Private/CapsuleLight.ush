// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FCapsuleLight
{
	float3	LightPos[2];
	float	Length;
	float	Radius;
	float	SoftRadius;
	float	DistBiasSqr;
};

void ClipToHorizon( inout float3 Line0, inout float3 Line1, float3 N )
{
	float NoP0 = dot( N, Line0 );
	float NoP1 = dot( N, Line1 );
	if( NoP0 < 0 ) Line0 = (  Line0 * NoP1 - Line1 * NoP0 ) / (  NoP1 - NoP0 );
	if( NoP1 < 0 ) Line1 = ( -Line0 * NoP1 + Line1 * NoP0 ) / ( -NoP1 + NoP0 );
}

// Closest point on line segment to point
float3 ClosestPointLineToPoint( float3 Line0, float3 Line1, float Length )
{
	float3 Line01 = Line1 - Line0;
	return Line0 + Line01 * saturate( -dot( Line01, Line0 ) / Pow2( Length ) );
}

// Closest point on line segment to ray
float3 ClosestPointLineToRay( float3 Line0, float3 Line1, float Length, float3 R )
{
	float3 L0 = Line0;
	float3 L1 = Line1;
	float3 Line01 = Line1 - Line0;

	// Shortest distance
	float A = Square( Length );
	float B = dot( R, Line01 );
	float t = saturate( dot( Line0, B*R - Line01 ) / (A - B*B) );

	return Line0 + t * Line01;
}

float3 SmallestAnglePointOnLineToRay( float3 Line0, float3 Line1, float Length, float3 R )
{
	float3 L0 = Line0;
	float3 L1 = Line1;
	float3 Line01 = Line1 - Line0;

	float A = Square( Length );
	float B = 2 * dot( Line0, Line01 );
	float C = dot( Line0, Line0 );
	float D = dot( R, Line0 );
	float E = dot( R, Line01 );
	float t = saturate( (B*D - 2*C*E) / (B*E - 2*A*D) );

	return Line0 + t * Line01;
}

void LineIrradiance( float3 Line0, float3 Line1, float DistanceBiasSqr, out float CosSubtended, out float Falloff, out half3 L )
{
	float LengthSqr0 = dot( Line0, Line0 );
	float LengthSqr1 = dot( Line1, Line1 );
	float InvLength0 = rsqrt( LengthSqr0 );
	float InvLength1 = rsqrt( LengthSqr1 );
	float InvLength01 = InvLength0 * InvLength1;

	CosSubtended = dot( Line0, Line1 ) * InvLength01;
	Falloff = InvLength01 / ( CosSubtended * 0.5 + 0.5 + DistanceBiasSqr * InvLength01 );
	L = 0.5 * ( Line0 * InvLength0 + Line1 * InvLength1 );
}

// Alpha is half of angle of spherical cap
float SphereHorizonCosWrap( float NoL, float SinAlphaSqr )
{
#if 1
	float SinAlpha = sqrt( SinAlphaSqr );

	if( NoL < SinAlpha )
	{
		NoL = max( NoL, -SinAlpha );
#if 0
		// Accurate sphere irradiance
		float CosBeta = NoL;
		float SinBeta = sqrt( 1 - CosBeta * CosBeta );
		float TanBeta = SinBeta / CosBeta;

		float x = sqrt( 1 / SinAlphaSqr - 1 );
		float y = -x / TanBeta;
		float z = SinBeta * sqrt(1 - y*y);

		NoL = NoL * acos(y) - x * z + atan( z / x ) / SinAlphaSqr;
		NoL /= PI;
#else
		// Hermite spline approximation
		// Fairly accurate with SinAlpha < 0.8
		// y=0 and dy/dx=0 at -SinAlpha
		// y=SinAlpha and dy/dx=1 at SinAlpha
		NoL = Pow2( SinAlpha + NoL ) / ( 4 * SinAlpha );
#endif
	}
#else
	NoL = saturate( ( NoL + SinAlphaSqr ) / ( 1 + SinAlphaSqr ) );
#endif

	return NoL;
}