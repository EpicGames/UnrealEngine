// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeferredShadingCommon.ush"
#include "MonteCarlo.ush"
#include "AreaLightCommon.ush"
#include "ShadingModels.ush"
#include "CapsuleLight.ush"
#include "CapsuleLightSampling.ush"
#include "SobolRandom.ush"

float IntegrateLight( FCapsuleLight Capsule, bool bInverseSquared )
{
	float Falloff;

	BRANCH
	if( Capsule.Length > 0 )
	{
		float LineCosSubtended = 1;
		half3 L;
		LineIrradiance( Capsule.LightPos[0], Capsule.LightPos[1], Capsule.DistBiasSqr, LineCosSubtended, Falloff, L );
	}
	else
	{
		float3 ToLight = Capsule.LightPos[0];
		float DistSqr = dot( ToLight, ToLight );
		Falloff = rcp( DistSqr + Capsule.DistBiasSqr );
	}

	Falloff = bInverseSquared ? Falloff : 1;

	return Falloff;
}

FAreaLight CreateAreaLight(float Roughness, half3 N, half3 V, FCapsuleLight Capsule, bool bInverseSquared )
{
	FAreaLight AreaLight = InitAreaLight();

	// Clip to horizon
	//float NoP0 = dot( N, Capsule.LightPos[0] );
	//float NoP1 = dot( N, Capsule.LightPos[1] );
	//if( NoP0 < 0 ) Capsule.LightPos[0] = (  Capsule.LightPos[0] * NoP1 - Capsule.LightPos[1] * NoP0 ) / ( NoP1 - NoP0);
	//if( NoP1 < 0 ) Capsule.LightPos[1] = ( -Capsule.LightPos[0] * NoP1 + Capsule.LightPos[1] * NoP0 ) / (-NoP1 + NoP0);

	BRANCH
	if( Capsule.Length > 0 )
	{
		LineIrradiance( Capsule.LightPos[0], Capsule.LightPos[1], Capsule.DistBiasSqr, AreaLight.LineCosSubtended, AreaLight.Falloff, AreaLight.DiffuseL );
		AreaLight.NoL = dot( N, AreaLight.DiffuseL );
		AreaLight.DiffuseL = normalize( AreaLight.DiffuseL );
	}
	else
	{
		float DistSqr = dot( Capsule.LightPos[0], Capsule.LightPos[0] );
		AreaLight.Falloff = rcp( DistSqr + Capsule.DistBiasSqr );

		AreaLight.DiffuseL = Capsule.LightPos[0] * rsqrt( DistSqr );
		AreaLight.NoL = dot( N, AreaLight.DiffuseL );
	}

	if( Capsule.Radius > 0 )
	{
		// TODO Use capsule area?
		float SinAlphaSqr = saturate( Pow2( Capsule.Radius ) * AreaLight.Falloff );
		AreaLight.NoL = SphereHorizonCosWrap( AreaLight.NoL, SinAlphaSqr );
	}

	AreaLight.NoL = saturate( AreaLight.NoL );
	AreaLight.Falloff = bInverseSquared ? AreaLight.Falloff : 1;

	float3 ToLight = Capsule.LightPos[0];
	if( Capsule.Length > 0 )
	{
		float3 R = reflect( -V, N );

		#if 0
			// Fix hard edge when ray is nearly parallel to line
			float3 PointOnLine = ClosestPointLineToPoint( Capsule.LightPos[0], Capsule.LightPos[1], Capsule.Length );
			float3 DirToLine = normalize( PointOnLine );
			R = lerp( DirToLine, R, saturate( dot( DirToLine, R ) ) );
			R = normalize( R );
		#endif

		ToLight = ClosestPointLineToRay( Capsule.LightPos[0], Capsule.LightPos[1], Capsule.Length, R );
	}

	float DistSqr = dot( ToLight, ToLight );
	float InvDist = rsqrt( DistSqr );
	AreaLight.SpecularL = ToLight * InvDist;
	
	float a = Pow2( Roughness );
    
	// Diffuse micro reflection contribution will softly fade out when th light become an area light.
	// We only based this assumption based on the light size.
	const float SizeFadesOutDiffuseMicroRefl = 20.0;

	AreaLight.SphereSinAlpha = saturate( Capsule.Radius * InvDist * (1 - a) );
	AreaLight.SphereSinAlphaSoft = saturate( Capsule.SoftRadius * InvDist );

	SetIsRectLight(AreaLight, false);
	SetAreaLightDiffuseMicroReflWeight(AreaLight, saturate(1.0f - max(Capsule.Length, Capsule.Radius) / SizeFadesOutDiffuseMicroRefl));

	return AreaLight;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting IntegrateBxDF( FGBufferData GBuffer, half3 N, half3 V, FCapsuleLight Capsule, FShadowTerms Shadow, bool bInverseSquared )
{
	GBuffer.Roughness = max( GBuffer.Roughness, View.MinRoughness );
	FAreaLight AreaLight = CreateAreaLight(GBuffer.Roughness, N, V, Capsule, bInverseSquared);
	return IntegrateBxDF( GBuffer, N, V, AreaLight, Shadow );
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting IntegrateBxDF( FGBufferData GBuffer, half3 N, half3 V, FCapsuleLight Capsule, FShadowTerms Shadow, uint2 SVPos )
{
	FDirectLighting Lighting = (FDirectLighting)0;

	Capsule.Radius = max( 1, Capsule.Radius );

	const float SphereArea = (4*PI) * Pow2( Capsule.Radius );
	const float CylinderArea = (2*PI) * Capsule.Radius * Capsule.Length;
	const float SurfaceArea = SphereArea + CylinderArea;
	const float SurfaceColor = 4.0 / SurfaceArea;

	float3 ToLight = 0.5 * ( Capsule.LightPos[0] + Capsule.LightPos[1] );
	float3 CapsuleAxis = normalize( Capsule.LightPos[1] - Capsule.LightPos[0] );

	float DistanceSqr = dot( ToLight, ToLight );
	float3 ConeAxis = ToLight * rsqrt( DistanceSqr );
	float SineConeSqr = saturate(Pow2(Capsule.Radius) / DistanceSqr);

	FCapsuleSphericalBounds CapsuleBounds = CapsuleGetSphericalBounds(ToLight, CapsuleAxis, Capsule.Radius, Capsule.Length);
	
	const uint NumSets = 3;
	const uint NumSamples[ NumSets ] =
	{
		0,	// Cosine hemisphere
		16,	// GGX
		16,	// Light area
	};
	
	UNROLL
	for( uint Set = 0; Set < NumSets; Set++ )
	{
		LOOP
		for( uint i = 0; i < NumSamples[ Set ]; i++ )
		{
			FRandomSequence RandSequence = RandomSequenceCreate( uint3( SVPos.xy, View.StateFrameIndex ), i, NumSamples[ Set ] );
			RandSequence.SampleSeed = StrongIntegerHash( Set );

			float2 E = RandSequence.Get2D();
			
			float3 L, H;
			if( Set == 0 )
			{
				L = TangentToWorld( CosineSampleHemisphere( E ).xyz, N );
				H = normalize(V + L);
			}
			else if( Set == 1 )
			{
				H = TangentToWorld( ImportanceSampleGGX( E, Pow4(GBuffer.Roughness) ).xyz, N );
				L = 2 * dot( V, H ) * H - V;
			}
			else
			{
				/*if( SourceLength > 0 )
				{
					uint2 Random = Rand3DPCG16( uint3( SVPos.xy, View.Random + i ) ).xy;

					float3 ToArea = ToLight;
					float3x3 Basis = GetTangentBasis( LightData.Tangent );
					
					if( ( (float)Random.x / 0xffff ) * SurfaceArea < SphereArea )
					{
						// Sphere caps
						float3 SpherePos = SourceRadius * UniformSampleSphere( E ).xyz;

						ToArea += mul( SpherePos, Basis );
						ToArea += LightData.Tangent * ( SpherePos.z > 0 ? 0.5 * SourceLength : -0.5 * SourceLength;
					}
					else
					{
						// Cylinder
						float Phi = (2*PI) * E.x;

						float3 CylinderPos;
						CylinderPos.x = SourceRadius * cos( Phi );
						CylinderPos.y = SourceRadius * sin( Phi );
						CylinderPos.z = SourceLength * E.z;

						ToArea += mul( CylinderPos, Basis );
					}

					L = normalize( ToArea );
					H = normalize( V + L );
				}
				else
				{
					uint2 Random = Rand3DPCG16( uint3( SVPos.xy, View.Random + i ) ).xy;

					float3 L01 = LightData.Tangent * SourceLength;
					float3 L0 = ToLight - 0.5 * L01;
					float3 L1 = ToLight + 0.5 * L01;
					L = lerp( L0, L1, (float)Random.x / 0xffff );
				
					float DistanceSqr = dot( L, L );
					float3 ConeAxis = L * rsqrt( DistanceSqr );
					float ConeCos = sqrt( 1 - Square( SourceRadius ) / DistanceSqr );

					L = TangentToWorld( UniformSampleCone( E, ConeCos ).xyz, ConeAxis );
					H = normalize(V + L);
				}*/

				if( Capsule.Length > 0 )
				{
					float3 ToArea = SampleCapsuleBounds(CapsuleBounds, E).xyz;
					L = normalize( ToArea );
					H = normalize( V + L );
				}
				else
				{
					L = TangentToWorld( UniformSampleConeRobust( E, SineConeSqr).xyz, ConeAxis );
					H = normalize(V + L);
				}
			}

			float NoL = saturate( dot(N, L) );
			float NoH = saturate( dot(N, H) );
			float VoH = saturate( dot(V, H) );

			//if( NoL > 0 && VoH > 0 )
			if( VoH > 0 )
			{
				{
					float3 ToSphere = ClosestPointLineToRay( Capsule.LightPos[0], Capsule.LightPos[1], Capsule.Length, L );

					if( length2( cross( L, ToSphere ) ) > Pow2( Capsule.Radius ) )
					{
						// Ray misses sphere
						continue;
					}
				}

				float PDF[] =
				{
					NoL / PI,
					D_GGX(Pow4(GBuffer.Roughness), NoH) * NoH / (4 * VoH),
					rcp(GetCapsuleBoundsInversePdf(L, CapsuleBounds))
				};

				if( Capsule.Length == 0 )
				{
					PDF[2] = 1.0 / UniformConeSolidAngle(SineConeSqr);
				}

				// MIS power heuristic
				float InvWeight = 0;
				UNROLL for( uint j = 0; j < NumSets; j++ )
				{
					InvWeight += Square( PDF[j] * NumSamples[j] );
				}
				float Weight = rcp( InvWeight ) * PDF[Set] * NumSamples[Set];
				
				FDirectLighting LightingSample = EvaluateBxDF( GBuffer, N, V, L, NoL, Shadow );

				Lighting.Diffuse		+= SurfaceColor * Weight * LightingSample.Diffuse;
				Lighting.Specular		+= SurfaceColor * Weight * LightingSample.Specular;
				Lighting.Transmission	+= SurfaceColor * Weight * LightingSample.Transmission;
			}
		}
	}

	return Lighting;
}