// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RectLight.ush"

float SqrtOneMinusX(float x)
{
	return x < 0.01 ? 1 - x * (0.5 + x * 0.125) : sqrt(1 - x);
}

// The footprint of a capsule can be bounded by either a rectangle or a cone
// This class holds both and will do sampling from whichever has the smallest solidangle
struct FCapsuleSphericalBounds
{
	FSphericalRect SphericalRect;
	float3 ConeAxis;
	float ConeSinThetaMax2;
	float ConeSolidAngle;
};

float GetCapsuleBoundsInversePdf(float3 Direction, FCapsuleSphericalBounds Bounds)
{
	if (Bounds.ConeSolidAngle < Bounds.SphericalRect.SolidAngle)
	{
		return Bounds.ConeSolidAngle;
	}
	float LocalDirZ = dot(Direction, Bounds.SphericalRect.Axis[2]);
	float DistanceSquared = Square(Bounds.SphericalRect.z0 / LocalDirZ);
	return GetSphericalRectInversePdf(Direction, DistanceSquared, Bounds.SphericalRect);
}

// return world space direction within the cone and solidangle
float4 SampleCapsuleBounds(FCapsuleSphericalBounds Bounds, float2 E)
{
	if (Bounds.ConeSolidAngle < Bounds.SphericalRect.SolidAngle)
	{
		return float4(TangentToWorld(UniformSampleConeRobust(E, Bounds.ConeSinThetaMax2).xyz, Bounds.ConeAxis), Bounds.ConeSolidAngle);
	}
	else
	{
		FSphericalRectSample Result = UniformSampleSphericalRect(E, Bounds.SphericalRect);
		return float4(Result.Direction, Result.InvPdf);
	}
}

// Given a bounding rectangle of a capsule, compute a bounding cone
FCapsuleSphericalBounds CapsuleGetSphericalBounds(float3 Origin, float3 Axis, float Radius, float Length)
{
	float h = dot(Axis, Origin);
	float3 ClosestPointOnAxis = Origin - Axis * h;

	float DistanceToAxisSqr = dot(ClosestPointOnAxis, ClosestPointOnAxis);
	float RadiusSqr = Pow2(Radius);

	if (DistanceToAxisSqr <= RadiusSqr)
	{
		// we are inside the infinite cylinder - only one of the caps can be visible
		float3 CapCenter = Origin - Axis * Length * 0.5 * sign(h);
		float LightDistanceSquared = dot(CapCenter, CapCenter);
		float SinThetaMax2 = saturate(RadiusSqr / LightDistanceSquared);

		FCapsuleSphericalBounds Result;
		Result.SphericalRect = (FSphericalRect)0;
		Result.SphericalRect.SolidAngle = POSITIVE_INFINITY; // make sure we don't pick this
		Result.ConeAxis = normalize(CapCenter);
		Result.ConeSinThetaMax2 = SinThetaMax2;
		Result.ConeSolidAngle = UniformConeSolidAngle(SinThetaMax2);
		return Result;
	}

	// we are outside the infinite cylinder, build a bounding rectangle
	FRect Rect;
	Rect.Origin = Origin;
	Rect.Axis[1] = Axis;
	Rect.Axis[2] = normalize(-ClosestPointOnAxis);
	Rect.Axis[0] = cross(Rect.Axis[1], Rect.Axis[2]);

	float SinCylinderAngle = Radius * rsqrt(DistanceToAxisSqr);
	// width * cos(angle) = 2r 
	float RectRadius = Radius * rsqrt(1 - Pow2(SinCylinderAngle));

	float Extension[2];
	for (int i = 0; i < 2; i++)
	{
		float hi = Length * (i > 0 ? 0.5 : -0.5);
		float3 PointPos = Origin + Axis * hi;

		float InverseDist = rsqrt(dot(PointPos, PointPos));

		float SinSphereAngle = saturate(Radius * InverseDist);
		float CosSphereAngle = SqrtOneMinusX(Pow2(SinSphereAngle));

		float CosAxisAngle = -dot(Axis, PointPos) * InverseDist;
		CosAxisAngle = sign(CosAxisAngle * hi) * saturate(abs(CosAxisAngle));
		float SinAxisAngle = SqrtOneMinusX(Pow2(CosAxisAngle));

		float CosExtension = SinAxisAngle * CosSphereAngle + CosAxisAngle * SinSphereAngle;
		Extension[i] = Radius / CosExtension;
	}

	float Translate = 0.5 * (Extension[1] - Extension[0]);
	float Extend = 0.5 * (Extension[0] + Extension[1]);

	Rect.Origin += Translate * Rect.Axis[1];
	Rect.Extent = float2(RectRadius, 0.5 * Length + Extend);

	// get a bounding cone from the bounding rectangle
	float3 R0 = Rect.Origin - Rect.Axis[1] * Rect.Extent.y;
	float3 R1 = Rect.Origin + Rect.Axis[1] * Rect.Extent.y;
	float InvDistR0 = rsqrt(dot(R0, R0));
	float InvDistR1 = rsqrt(dot(R1, R1));

	FCapsuleSphericalBounds Result;
	Result.SphericalRect = BuildSphericalRect(Rect);
	Result.ConeAxis = normalize(lerp(R0, R1, saturate(InvDistR1 / (InvDistR0 + InvDistR1))));
	Result.ConeSinThetaMax2 = saturate(0.5 - 0.5 * dot(R0, R1) * InvDistR0 * InvDistR1); // sin(x/2)^2 = (1-cos(x))/2
	Result.ConeSolidAngle = UniformConeSolidAngle(Result.ConeSinThetaMax2);
	return Result;
}

// Adapted from:
//    https://www.iquilezles.org/www/articles/intersectors/intersectors.htm
// This code assume the ray direction is normalized and only returns the front hit
float CapsuleIntersect(float3 Rd, float3 Center, float3 Axis, float Radius2, float Length)
{
	float3 ba = Axis;
	float3 oa = -Center;
	float bard = dot(ba, Rd);
	float baoa = dot(ba, oa);
	float rdoa = dot(Rd, oa);
	float oaoa = dot(oa, oa);
	float a = 1 - bard * bard;
	float b = rdoa - baoa * bard;
	float c = oaoa - baoa * baoa - Radius2;
	float h = b * b - a * c;
	if (h >= 0.0)
	{
		float t = (-b - sqrt(h)) / a;
		float y = baoa + t * bard;
		// body
		if (abs(y) < 0.5 * Length)
		{
			return t;
		}

		// caps
		float3 oc = oa - (sign(y) * 0.5 * Length) * ba;
		b = dot(Rd, oc);
		h = Radius2 - length2(oc - b * Rd);
		if (h > 0.0)
		{
			return -b - sqrt(h);
		}
	}
	return -1.0;
}

// simpler intersection test for rays that are likely to intersect
// returned intersection distance is approximate
float CapsuleTest(float3 Rd, float3 Center, float3 Axis, float Radius2, float Length)
{
	// Shortest distance
	float B = dot(Rd, Axis);
	float t = clamp(dot(Center, B * Rd - Axis) / (1 - B * B), -0.5 * Length, 0.5 * Length);
	float3 ToSphere = Center + t * Axis;
	float3 C = cross(Rd, ToSphere);
	return dot(C, C) <= Radius2 ? length(ToSphere) : -1.0;
}
