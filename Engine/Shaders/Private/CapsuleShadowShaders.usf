// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CapsuleShadowShaders.usf: Tiled deferred culling and shadowing from capsule shapes
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "FastMath.ush"
#include "DistanceFieldLightingShared.ush"
#include "SHCommon.ush"
#include "VolumetricLightmapShared.ush"
#include "ReflectionEnvironmentShared.ush"
#include "IntersectionUtils.ush"
#include "Substrate/Substrate.ush"

#ifdef UPSAMPLE_PASS
#	include "ShadowFactorsUpsampleCommon.ush"
#endif

#ifndef THREADGROUP_SIZEX
#	define THREADGROUP_SIZEX 1
#endif

#ifndef THREADGROUP_SIZEY
#	define THREADGROUP_SIZEY 1
#endif

#ifndef LIGHT_SOURCE_MODE
#define LIGHT_SOURCE_MODE 0
#endif

// must match CapsuleShadowRendering.cpp
#define LIGHT_SOURCE_PUNCTUAL 0
#define LIGHT_SOURCE_FROM_CAPSULE 1
#define LIGHT_SOURCE_FROM_RECEIVER 2

struct FCapsuleShape
{
	float3 TranslatedCenter;
	float Radius;
	float3 Orientation;
	float Length;
};

// SUPPORT_CAPSULE_SHAPES

	/** Number of capsules affecting the tile, after culling. */
	groupshared uint TileNumCapsules0;
	groupshared uint TileNumCapsules1;

	#define MAX_INTERSECTING_SHAPES 512
	groupshared uint IntersectingShapeIndices[MAX_INTERSECTING_SHAPES * 2];

	uint NumShadowCapsules;
	StructuredBuffer<FCapsuleShape> ShadowCapsuleShapes;


// SUPPORT_MESH_DISTANCE_FIELDS

	/** Number of distance fields affecting the tile, after culling. */
	groupshared uint TileNumDistanceFields0;
	groupshared uint TileNumDistanceFields1;

	#define MAX_INTERSECTING_DISTANCE_FIELDS 64
	groupshared uint IntersectingMeshDistanceFieldIndices[MAX_INTERSECTING_DISTANCE_FIELDS * 2];

	uint NumMeshDistanceFieldCasters;
	StructuredBuffer<uint> MeshDistanceFieldCasterIndices;


#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE
StructuredBuffer<float4> LightDirectionData;

void DecodeLightDirectionW(float LightDirectionW, out float LightAngle, out float MinVisibility)
{
	uint WInt = asuint(LightDirectionW);
	LightAngle = f16tof32(WInt);
	MinVisibility = f16tof32(WInt >> 16);
}

float EncodeLightDirectionW(float LightAngle, float MinVisibility)
{
	return asfloat(f32tof16(LightAngle) | f32tof16(MinVisibility) << 16);
}

void GetLightDirectionData(uint ShapeIndex, bool bDistanceFieldCaster, out float3 LightDirection, out float LightAngle, out float MinVisibility)
{
	// Light data for distance field casters is placed after light data for capsules in SetupIndirectCapsuleShadows
	uint BaseLightDataIndex = bDistanceFieldCaster ? NumShadowCapsules : 0;

	float4 VectorValue = LightDirectionData[ShapeIndex + BaseLightDataIndex];
	LightDirection = VectorValue.xyz;
	DecodeLightDirectionW(VectorValue.w, LightAngle, MinVisibility);
}

uint SkyLightMode;
float CapsuleIndirectConeAngle;
float CapsuleSkyAngleScale;
float CapsuleMinSkyAngle;
uint NumLightDirectionData;

RWStructuredBuffer<float4> RWComputedLightDirectionData;

[numthreads(THREADGROUP_SIZEX, 1, 1)]
void ComputeLightDirectionFromVolumetricLightmapCS(
	uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x < NumLightDirectionData)
	{
		float4 LightData = LightDirectionData[DispatchThreadId.x];
		float3 ObjectPosition = LightData.xyz;
		float3 BrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(ObjectPosition);
		float3 LightDirection;
		float LightAngle;

		if (SkyLightMode == 2)
		{
			FTwoBandSHVectorRGB SkyIrradianceSH;
			// See ComputeSkyEnvMapDiffuseIrradianceCS for the coefficient and sign adaptation. N need to rescale the coefficients, direction is preserve.
			SkyIrradianceSH.R.V = SkyIrradianceEnvironmentMap[0].wyzx * float4(1.0f, -1.0f, 1.0f, -1.0f);
			SkyIrradianceSH.G.V = SkyIrradianceEnvironmentMap[1].wyzx * float4(1.0f, -1.0f, 1.0f, -1.0f);
			SkyIrradianceSH.B.V = SkyIrradianceEnvironmentMap[2].wyzx * float4(1.0f, -1.0f, 1.0f, -1.0f);
			LightDirection = GetMaximumDirection(GetLuminance(SkyIrradianceSH));
			LightAngle = CapsuleIndirectConeAngle;
		}
		else if (SkyLightMode == 1)
		{
			// Stationary sky light shadowing
			float3 SkyBentNormal = GetVolumetricLightmapSkyBentNormal(BrickTextureUVs);
			float SkyBentNormalLength = length(SkyBentNormal);

			float ConeAngle = max(SkyBentNormalLength * CapsuleSkyAngleScale * .5f * PI, CapsuleMinSkyAngle * PI / 180.0f);
			LightDirection = SkyBentNormal / max(SkyBentNormalLength, .0001f);
			LightAngle = ConeAngle;
		}
		else
		{
			FTwoBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH2(BrickTextureUVs);
			LightDirection = GetMaximumDirection(GetLuminance(IrradianceSH));
			LightAngle = CapsuleIndirectConeAngle;
		}
		
		if (dot(LightDirection, LightDirection) < .1f)
		{
			LightDirection = float3(0, 0, 1);
		}

		float Unused;
		float MinVisibility;
		DecodeLightDirectionW(LightData.w, Unused, MinVisibility);

		float4 PackedLightDirection = float4(LightDirection, EncodeLightDirectionW(LightAngle, MinVisibility));
		RWComputedLightDirectionData[DispatchThreadId.x] = PackedLightDirection;
	}
}

#endif

#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL
/** From point being shaded toward light, for directional lights. */
float3 LightDirection;
float4 LightTranslatedPositionAndInvRadius;
float LightSourceRadius;
float RayStartOffsetDepthScale;
float3 LightAngleAndNormalThreshold;
#endif

uint4 ScissorRectMinAndSize;
float2 NumGroups;

/** Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ;
groupshared uint IntegerTileMaxZ;

/** Inner Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ2;
groupshared uint IntegerTileMaxZ2;

struct FTileCullingData
{
	float4 TranslatedBoundingSphere;
#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL
	float3 ConeAxis;
	float ConeAngleCos;
	float ConeAngleSin;
#endif
};

void SetupTileCullingData(
	float SceneDepth, 
	float MaxDepth,
	uint ThreadIndex, 
	uint2 GroupId, 
	out FTileCullingData TileCullingData0,
	out FTileCullingData TileCullingData1,
	out bool bTileShouldComputeShadowing,
	out uint GroupIndex)
{
	// Initialize per-tile variables
    if (ThreadIndex == 0) 
	{
        IntegerTileMinZ = 0x7F7FFFFF;     
        IntegerTileMaxZ = 0;
		IntegerTileMinZ2 = 0x7F7FFFFF;  
		IntegerTileMaxZ2 = 0;
		TileNumCapsules0 = 0;
		TileNumCapsules1 = 0;
		TileNumDistanceFields0 = 0;
		TileNumDistanceFields1 = 0;
    }

    GroupMemoryBarrierWithGroupSync();

	// Use shared memory atomics to build the depth bounds for this tile
	// Each thread is assigned to a pixel at this point
	//@todo - move depth range computation to a central point where it can be reused by all the frame's tiled deferred passes!

	if (SceneDepth < MaxDepth)
	{
		InterlockedMin(IntegerTileMinZ, asuint(SceneDepth));
		InterlockedMax(IntegerTileMaxZ, asuint(SceneDepth));
	}

    GroupMemoryBarrierWithGroupSync();

    float MinTileZ = asfloat(IntegerTileMinZ);
    float MaxTileZ = asfloat(IntegerTileMaxZ);

	float HalfZ = .5f * (MinTileZ + MaxTileZ);

	if (SceneDepth < MaxDepth)
	{
		// Compute a second min and max Z, clipped by HalfZ, so that we get two depth bounds per tile
		// This results in more conservative tile depth bounds and fewer intersections
		if (SceneDepth >= HalfZ)
		{
			InterlockedMin(IntegerTileMinZ2, asuint(SceneDepth));
		}

		if (SceneDepth <= HalfZ)
		{
			InterlockedMax(IntegerTileMaxZ2, asuint(SceneDepth));
		}
	}

	GroupMemoryBarrierWithGroupSync();
	
	float MinTileZ2 = asfloat(IntegerTileMinZ2);
	float MaxTileZ2 = asfloat(IntegerTileMaxZ2);

	bTileShouldComputeShadowing = true;

	if (IntegerTileMinZ == 0x7F7FFFFF && IntegerTileMaxZ == 0)
	{
		bTileShouldComputeShadowing = false;
	}

	float3 ViewTileMin;
	float3 ViewTileMax;

	float3 ViewTileMin2;
	float3 ViewTileMax2;

	bool bCenteredProjection = abs(View.ViewToClip[2][0]) < .00001f && abs(View.ViewToClip[2][1]) < .00001f;

	BRANCH
	// Off center projection path uses 37 more asm instructions
	if (bCenteredProjection)
	{
		float2 TanViewFOV = GetTanHalfFieldOfView();
		// tan(FOV) = HalfUnitPlaneWidth / 1, so TanViewFOV * 2 is the size of the whole unit view plane
		// We are operating on a subset of that defined by ScissorRectMinAndSize
		float2 TileSize = TanViewFOV * 2 * ScissorRectMinAndSize.zw / ((float2)View.ViewSizeAndInvSize.xy * NumGroups);
		float2 UnitPlaneMin = -TanViewFOV + TanViewFOV * 2 * (ScissorRectMinAndSize.xy - View.ViewRectMin.xy) * View.ViewSizeAndInvSize.zw;

		float2 UnitPlaneTileMin = (GroupId.xy * TileSize + UnitPlaneMin) * float2(1, -1);
		float2 UnitPlaneTileMax = ((GroupId.xy + 1) * TileSize + UnitPlaneMin) * float2(1, -1);

		ViewTileMin.xy = min(MinTileZ * UnitPlaneTileMin, MaxTileZ2 * UnitPlaneTileMin);
		ViewTileMax.xy = max(MinTileZ * UnitPlaneTileMax, MaxTileZ2 * UnitPlaneTileMax);
		ViewTileMin.z = MinTileZ;
		ViewTileMax.z = MaxTileZ2;
		ViewTileMin2.xy = min(MinTileZ2 * UnitPlaneTileMin, MaxTileZ * UnitPlaneTileMin);
		ViewTileMax2.xy = max(MinTileZ2 * UnitPlaneTileMax, MaxTileZ * UnitPlaneTileMax);
		ViewTileMin2.z = MinTileZ2;
		ViewTileMax2.z = MaxTileZ;
	}
	else
	{
		float2 TileSize = 2 * ScissorRectMinAndSize.zw / ((float2)View.ViewSizeAndInvSize.xy * NumGroups);
		float2 UnitPlaneMin = -1 + 2 * (ScissorRectMinAndSize.xy - View.ViewRectMin.xy) * View.ViewSizeAndInvSize.zw;

		float2 UnitPlaneTileMin = (GroupId.xy * TileSize + UnitPlaneMin) * float2(1, -1);
		float2 UnitPlaneTileMax = ((GroupId.xy + 1) * TileSize + UnitPlaneMin) * float2(1, -1);

		{
			float MinTileDeviceZ = ConvertToDeviceZ(MinTileZ);
			float4 MinDepthMinCorner = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MinTileDeviceZ, 1), View.ClipToView);
			float4 MinDepthMaxCorner = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MinTileDeviceZ, 1), View.ClipToView);
	
			float MaxTileDeviceZ = ConvertToDeviceZ(MaxTileZ2);
			float4 MaxDepthMinCorner = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MaxTileDeviceZ, 1), View.ClipToView);
			float4 MaxDepthMaxCorner = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MaxTileDeviceZ, 1), View.ClipToView);

			ViewTileMin.xy = min(MinDepthMinCorner.xy / MinDepthMinCorner.w, MaxDepthMinCorner.xy / MaxDepthMinCorner.w);
			ViewTileMax.xy = max(MinDepthMaxCorner.xy / MinDepthMaxCorner.w, MaxDepthMaxCorner.xy / MaxDepthMaxCorner.w);
			ViewTileMin.z = MinTileZ;
			ViewTileMax.z = MaxTileZ2;
		}

		{
			float MinTileDeviceZ = ConvertToDeviceZ(MinTileZ2);
			float4 MinDepthMinCorner = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MinTileDeviceZ, 1), View.ClipToView);
			float4 MinDepthMaxCorner = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MinTileDeviceZ, 1), View.ClipToView);
	
			float MaxTileDeviceZ = ConvertToDeviceZ(MaxTileZ);
			float4 MaxDepthMinCorner = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MaxTileDeviceZ, 1), View.ClipToView);
			float4 MaxDepthMaxCorner = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MaxTileDeviceZ, 1), View.ClipToView);

			ViewTileMin2.xy = min(MinDepthMinCorner.xy / MinDepthMinCorner.w, MaxDepthMinCorner.xy / MaxDepthMinCorner.w);
			ViewTileMax2.xy = max(MinDepthMaxCorner.xy / MinDepthMaxCorner.w, MaxDepthMaxCorner.xy / MaxDepthMaxCorner.w);
			ViewTileMin2.z = MinTileZ2;
			ViewTileMax2.z = MaxTileZ;
		}
	}

	float3 ViewGroup0Center = (ViewTileMax + ViewTileMin) / 2;
	TileCullingData0.TranslatedBoundingSphere.xyz = mul(float4(ViewGroup0Center, 1), View.ViewToTranslatedWorld).xyz;
	TileCullingData0.TranslatedBoundingSphere.w = length(ViewGroup0Center - ViewTileMax);

	float3 ViewGroup1Center = (ViewTileMax2 + ViewTileMin2) / 2;
	TileCullingData1.TranslatedBoundingSphere.xyz = mul(float4(ViewGroup1Center, 1), View.ViewToTranslatedWorld).xyz;
	TileCullingData1.TranslatedBoundingSphere.w = length(ViewGroup1Center - ViewTileMax);

#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL
	#if POINT_LIGHT
		float3 LightVector0 = LightTranslatedPositionAndInvRadius.xyz - TileCullingData0.TranslatedBoundingSphere.xyz;
		float LightVector0Length = length(LightVector0);
		float3 LightVector1 = LightTranslatedPositionAndInvRadius.xyz - TileCullingData1.TranslatedBoundingSphere.xyz;
		float LightVector1Length = length(LightVector1);
		TileCullingData0.ConeAxis = LightVector0 / LightVector0Length;
		TileCullingData1.ConeAxis = LightVector1 / LightVector1Length;;
		float TanLightAngle0 = LightSourceRadius / LightVector0Length;
		float TanLightAngle1 = LightSourceRadius / LightVector1Length;

		TileCullingData0.ConeAngleCos = 1.0f / sqrt(1 + TanLightAngle0 * TanLightAngle0);
		TileCullingData0.ConeAngleSin = TileCullingData0.ConeAngleCos * TanLightAngle0;
	
		TileCullingData1.ConeAngleCos = 1.0f / sqrt(1 + TanLightAngle1 * TanLightAngle1);
		TileCullingData1.ConeAngleSin = TileCullingData1.ConeAngleCos * TanLightAngle1;

		// Don't operate on tiles completely outside of the light's influence
		bool bTileInLightInfluenceBounds = LightVector0Length < 1.0f / LightTranslatedPositionAndInvRadius.w + TileCullingData0.TranslatedBoundingSphere.w
			|| LightVector1Length < 1.0f / LightTranslatedPositionAndInvRadius.w + TileCullingData1.TranslatedBoundingSphere.w;

		bTileShouldComputeShadowing = bTileShouldComputeShadowing && bTileInLightInfluenceBounds;

	#else
		TileCullingData0.ConeAxis = TileCullingData1.ConeAxis = LightDirection;
		TileCullingData0.ConeAngleCos = TileCullingData1.ConeAngleCos = cos(LightAngleAndNormalThreshold.x);
		TileCullingData0.ConeAngleSin = TileCullingData1.ConeAngleSin = sin(LightAngleAndNormalThreshold.x);
	#endif
#endif

	GroupIndex = SceneDepth > MaxTileZ2 ? 1 : 0;
}

// Scaled sphere intersection allows capsule shadows to blend together better when penumbras are large, so use for indirect.
// Otherwise an occluder sphere will be extracted from the capsule and used for shadowing.  
// This maintains shadow silhouette shapes better but has a discontinuity when the capsule direction is nearly parallel to the light direction.
#define USE_SCALED_SPHERE_INTERSECTION (LIGHT_SOURCE_MODE != LIGHT_SOURCE_PUNCTUAL)

uint CullCapsuleShapesToTile(
	uint ThreadIndex,
	uint GroupIndex, 
	float MaxOcclusionDistance,
	FTileCullingData TileCullingData0,
	FTileCullingData TileCullingData1)
{
#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL

	const float3 ConeAxis0 = TileCullingData0.ConeAxis;
	const float ConeAngleCos0 = TileCullingData0.ConeAngleCos;
	const float ConeAngleSin0 = TileCullingData0.ConeAngleSin;
	const float3 ConeAxis1 = TileCullingData1.ConeAxis;
	const float ConeAngleCos1 = TileCullingData1.ConeAngleCos;
	const float ConeAngleSin1 = TileCullingData1.ConeAngleSin;

#endif

	LOOP
	for (uint ShapeIndex = ThreadIndex; ShapeIndex < NumShadowCapsules; ShapeIndex += THREADGROUP_SIZEX * THREADGROUP_SIZEY)
	{
		#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE

			float3 ConeAxis0;
			float LightAngle;
			float Unused;
			GetLightDirectionData(ShapeIndex, false, ConeAxis0, LightAngle, Unused);
			
			float ConeAngleCos0 = cos(LightAngle);
			float ConeAngleSin0 = sin(LightAngle);

			float3 ConeAxis1 = ConeAxis0;
			float ConeAngleCos1 = ConeAngleCos0;
			float ConeAngleSin1 = ConeAngleSin0;

		#endif
		
		FCapsuleShape CapsuleShape = ShadowCapsuleShapes[ShapeIndex];

		float3 TransformedSphereTranslatedCenter = CapsuleShape.TranslatedCenter;
		float TransformedSphereRadius = CapsuleShape.Radius;
		float3 TransformedTileTranslatedBoundingSphereCenter0 = TileCullingData0.TranslatedBoundingSphere.xyz;
		float3 TransformedTileTranslatedBoundingSphereCenter1 = TileCullingData1.TranslatedBoundingSphere.xyz;

		#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
			float3 TransformedConeAxis0 = ConeAxis0;
			float3 TransformedConeAxis1 = ConeAxis1;
		#endif

		#if USE_SCALED_SPHERE_INTERSECTION
			float3 CapsuleSpaceX;
			float3 CapsuleSpaceY;
			float3 CapsuleSpaceZ = CapsuleShape.Orientation;
			GenerateCoordinateSystem(CapsuleSpaceZ, CapsuleSpaceX, CapsuleSpaceY);

			// Scale required along the capsule's axis to turn it into a sphere (assuming it was originally a scaled sphere instead of a capsule)
			float CapsuleZScale = CapsuleShape.Radius / (.5f * CapsuleShape.Length + CapsuleShape.Radius);
			CapsuleSpaceZ *= CapsuleZScale;

			// The capsule is centered at 0 in the scaled sphere space
			TransformedSphereTranslatedCenter = 0;

			// After scaling along the capsule axis it will become a sphere with the original radius
			TransformedSphereRadius = CapsuleShape.Radius;

			// Transform the sphere center and cone axis into the scaled sphere space
			float3 CapsuleCenterToTileCenter0 = TileCullingData0.TranslatedBoundingSphere.xyz - CapsuleShape.TranslatedCenter;
			TransformedTileTranslatedBoundingSphereCenter0 = float3(dot(CapsuleCenterToTileCenter0, CapsuleSpaceX), dot(CapsuleCenterToTileCenter0, CapsuleSpaceY), dot(CapsuleCenterToTileCenter0, CapsuleSpaceZ));
			
			float3 CapsuleCenterToTileCenter1 = TileCullingData1.TranslatedBoundingSphere.xyz - CapsuleShape.TranslatedCenter;
			TransformedTileTranslatedBoundingSphereCenter1 = float3(dot(CapsuleCenterToTileCenter1, CapsuleSpaceX), dot(CapsuleCenterToTileCenter1, CapsuleSpaceY), dot(CapsuleCenterToTileCenter1, CapsuleSpaceZ));

			#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
				// Renormalize the cone axis as it went through a non-uniformly scaled transform
				TransformedConeAxis0 = normalize(float3(dot(ConeAxis0, CapsuleSpaceX), dot(ConeAxis0, CapsuleSpaceY), dot(ConeAxis0, CapsuleSpaceZ)));
				TransformedConeAxis1 = normalize(float3(dot(ConeAxis1, CapsuleSpaceX), dot(ConeAxis1, CapsuleSpaceY), dot(ConeAxis1, CapsuleSpaceZ)));
			#endif
		#else
			// Add half capsule length to bounding sphere
			TransformedSphereRadius = CapsuleShape.Radius + .5f * CapsuleShape.Length;
		#endif

		BRANCH
		if (SphereIntersectSphere(float4(TransformedSphereTranslatedCenter, TransformedSphereRadius + MaxOcclusionDistance), float4(TransformedTileTranslatedBoundingSphereCenter0, TileCullingData0.TranslatedBoundingSphere.w))
			#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
				&& SphereIntersectConeWithMaxDistance(float4(TransformedSphereTranslatedCenter, TransformedSphereRadius + TileCullingData0.TranslatedBoundingSphere.w), TransformedTileTranslatedBoundingSphereCenter0, TransformedConeAxis0, ConeAngleCos0, ConeAngleSin0, MaxOcclusionDistance)
			#endif
			)
		{
			uint ListIndex;
			InterlockedAdd(TileNumCapsules0, 1U, ListIndex);
			// Don't overwrite on overflow
			ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_SHAPES - 1));
			IntersectingShapeIndices[MAX_INTERSECTING_SHAPES * 0 + ListIndex] = ShapeIndex; 
		}

		BRANCH
		if (SphereIntersectSphere(float4(TransformedSphereTranslatedCenter, TransformedSphereRadius + MaxOcclusionDistance), float4(TransformedTileTranslatedBoundingSphereCenter1, TileCullingData1.TranslatedBoundingSphere.w))
			#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
				&& SphereIntersectConeWithMaxDistance(float4(TransformedSphereTranslatedCenter, TransformedSphereRadius + TileCullingData1.TranslatedBoundingSphere.w), TransformedTileTranslatedBoundingSphereCenter1, TransformedConeAxis1, ConeAngleCos1, ConeAngleSin1, MaxOcclusionDistance)
			#endif
			)
		{
			uint ListIndex;
			InterlockedAdd(TileNumCapsules1, 1U, ListIndex);
			// Don't write out of bounds on overflow
			ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_SHAPES - 1));
			IntersectingShapeIndices[MAX_INTERSECTING_SHAPES * 1 + ListIndex] = ShapeIndex; 
		}
	}

	GroupMemoryBarrierWithGroupSync();

	return min(GroupIndex == 0 ? TileNumCapsules0 : TileNumCapsules1, (uint)MAX_INTERSECTING_SHAPES);
}

// Approximate the area of intersection of two spherical caps, from 'Ambient Aperture Lighting'
// fRadius0 : First caps radius (arc length in radians)
// fRadius1 : Second caps radius (in radians)
// fDist : Distance between caps (radians between centers of caps)
float SphericalCapIntersectionAreaFast(float fRadius0, float fRadius1, float fDist)
{
	float fArea;

	if ( fDist <= max(fRadius0, fRadius1) - min(fRadius0, fRadius1) )
	{
		// One cap is completely inside the other
		fArea = 6.283185308f - 6.283185308f * cos( min(fRadius0,fRadius1) );
	}
	else if ( fDist >= fRadius0 + fRadius1 )
	{
		// No intersection exists
		fArea = 0;
	}
	else
	{
		float fDiff = abs(fRadius0 - fRadius1);
		fArea = smoothstep(0.0f,
			1.0f,
			1.0f - saturate((fDist-fDiff)/(fRadius0+fRadius1-fDiff)));
			fArea *= 6.283185308f - 6.283185308f * cos( min(fRadius0,fRadius1) );
	}
	return fArea;
}

// CosFadeStartAngle in x, 1 / (1 - CosFadeStartAngle) in y
float2 CosFadeStartAngle;

void ApplyFadeToCapsuleRadius(inout float CapsuleRadius, float3 LightDirection)
{
	#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE
		// Fade out when nearly vertical up due to self shadowing artifacts
		float ShapeFadeAlpha = 1 - saturate(2 * (-LightDirection.z - CosFadeStartAngle.x) * CosFadeStartAngle.y);
		CapsuleRadius *= ShapeFadeAlpha;
	#endif
}

float ShadowConeTraceAgainstCulledCapsuleShapes(
	float3 TranslatedWorldRayStart, 
	float3 UnitRayDirection,
	float LightVectorLength,
	float LightAngle,  
	float InvMaxOcclusionDistance,
	uint CulledDataParameter, 
	uint NumIntersectingCapsules,
	uniform bool bUseCulling)
{
	float ConeVisibility = 1; 
	float AreaOfLight = 6.283185308f - 6.283185308f * cos(LightAngle);

	LOOP
	for (uint TileCulledObjectIndex = 0; TileCulledObjectIndex < NumIntersectingCapsules; TileCulledObjectIndex++)
	{
		uint ObjectIndex;

		if (bUseCulling)
		{
			uint GroupIndex = CulledDataParameter;
			ObjectIndex = IntersectingShapeIndices[MAX_INTERSECTING_SHAPES * GroupIndex + TileCulledObjectIndex];
		}
		else
		{
			ObjectIndex = TileCulledObjectIndex;
		}

		float MinVisibility = 0.0f;

		#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE
			GetLightDirectionData(ObjectIndex, false, UnitRayDirection, LightAngle, MinVisibility);
			AreaOfLight = 6.283185308f - 6.283185308f * cos(LightAngle);
		#endif

		#define OVERRIDE_LIGHT_DEBUG 0
		#if OVERRIDE_LIGHT_DEBUG
			//UnitRayDirection = normalize(float3(.2f, .2f, .8f));
			UnitRayDirection = float3(0, 0, 1);
			LightAngle = .3f;
		#endif
			
		FCapsuleShape CapsuleShape = ShadowCapsuleShapes[ObjectIndex];

		ApplyFadeToCapsuleRadius(CapsuleShape.Radius, UnitRayDirection);

		float DistanceToShadowSphere;
		float3 UnitVectorToShadowSphere;
		float3 UnitRayDirectionInCorrectSpace = UnitRayDirection;

		BRANCH
		if (CapsuleShape.Length > 0)
		{
#if USE_SCALED_SPHERE_INTERSECTION

			float3 CapsuleSpaceX;
			float3 CapsuleSpaceY;
			float3 CapsuleSpaceZ = CapsuleShape.Orientation;
			GenerateCoordinateSystem(CapsuleSpaceZ, CapsuleSpaceX, CapsuleSpaceY);

			float CapsuleZScale = CapsuleShape.Radius / (.5f * CapsuleShape.Length + CapsuleShape.Radius);
			CapsuleSpaceZ *= CapsuleZScale;

			float3 CapsuleCenterToRayStart = TranslatedWorldRayStart - CapsuleShape.TranslatedCenter;
			float3 CapsuleSpaceRayStart = float3(dot(CapsuleCenterToRayStart, CapsuleSpaceX), dot(CapsuleCenterToRayStart, CapsuleSpaceY), dot(CapsuleCenterToRayStart, CapsuleSpaceZ));

			float3 CapsuleSpaceRayDirection = float3(dot(UnitRayDirection, CapsuleSpaceX), dot(UnitRayDirection, CapsuleSpaceY), dot(UnitRayDirection, CapsuleSpaceZ));

			DistanceToShadowSphere = length(CapsuleSpaceRayStart);
			UnitVectorToShadowSphere = -CapsuleSpaceRayStart / DistanceToShadowSphere;
			UnitRayDirectionInCorrectSpace = normalize(CapsuleSpaceRayDirection);
#else
			float3 VectorToCapsuleCenter = CapsuleShape.TranslatedCenter - TranslatedWorldRayStart;

			// Closest point on line segment to ray
			float3 L01 = CapsuleShape.Orientation * CapsuleShape.Length;
			float3 L0 = VectorToCapsuleCenter - 0.5 * L01;
			float3 L1 = VectorToCapsuleCenter + 0.5 * L01;

			// The below is computing the shortest distance between capsule line segment and ray
			float CapsuleOrientationProjectedOntoRay = dot(UnitRayDirection, L01);
			// Vector that spans L01 perpendicular to the ray
			float3 PerpendicularSpanningVector = CapsuleOrientationProjectedOntoRay * UnitRayDirection - L01;
			// Length of PerpendicularSpanningVector using the right triangle formed by L01 and UnitRayDirection * CapsuleOrientationProjectedOntoRay
			float PerpendicularDistance = Square(CapsuleShape.Length) - CapsuleOrientationProjectedOntoRay * CapsuleOrientationProjectedOntoRay;
			// Project the vector to a capsule endpoint onto the perpendicular spanning vector, normalized
			float t = saturate(dot(L0, PerpendicularSpanningVector) / PerpendicularDistance);
			// Compute the vector to the shadow sphere which best approximates the capsule's shadowing 
			float3 VectorToShadowSphere = L0 + t * L01;

			DistanceToShadowSphere = length(VectorToShadowSphere);
			UnitVectorToShadowSphere = VectorToShadowSphere / DistanceToShadowSphere;
		
			// The above 'best shadow sphere' calculation doesn't take into account the projected solid angle of the potential shadow spheres
			// As a result, there's a discontinuity when the capsule and the ray point in nearly the same direction, where the far end of the capsule gets chosen
			// Here we mitigate the effect by overriding the distance to shadow sphere if one of the capsule end points was closer
			DistanceToShadowSphere = min(DistanceToShadowSphere, length(L0));
			DistanceToShadowSphere = min(DistanceToShadowSphere, length(L1));
#endif
		}
		else
		{
			DistanceToShadowSphere = length(CapsuleShape.TranslatedCenter - TranslatedWorldRayStart);
			UnitVectorToShadowSphere = (CapsuleShape.TranslatedCenter - TranslatedWorldRayStart) / DistanceToShadowSphere;
		}

		float AngleBetween = acosFast(dot(UnitVectorToShadowSphere, UnitRayDirectionInCorrectSpace));
		float IntersectionArea = SphericalCapIntersectionAreaFast(LightAngle, atanFastPos(CapsuleShape.Radius / DistanceToShadowSphere), AngleBetween);
		#if POINT_LIGHT
			// SphericalCapIntersectionAreaFast does not take the relative distance of the two sphere caps into account, which can cause shadows to be cast on the opposite direction
			// Here we compare DistanceToShadowSphere and LightVectorLength to determine whether a shadow should be cast
			// To prevent discontinuity, we use the ratio of the distance difference to the capsule's radius as a smooth factor
			IntersectionArea = lerp(IntersectionArea, 0, saturate((DistanceToShadowSphere - LightVectorLength + CapsuleShape.Radius) / CapsuleShape.Radius));
		#endif
		float ConeConeIntersection = 1 - saturate(IntersectionArea / AreaOfLight);
		float DistanceFadeAlpha = saturate(DistanceToShadowSphere * InvMaxOcclusionDistance * 3 - 2);
		ConeConeIntersection = lerp(ConeConeIntersection, 1, DistanceFadeAlpha);
		
		#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_PUNCTUAL
			// Apply to indirect shadows only
			ConeConeIntersection = lerp(MinVisibility, 1, ConeConeIntersection);
		#endif

		ConeVisibility *= ConeConeIntersection;
	}

	return ConeVisibility;
}

// Whether to actually sample the distance field while doing tile culling
#define USE_DISTANCE_FIELD_FOR_TILE_CULLING 1

bool TileBoundsIntersectDistanceFieldCaster(
	uint ObjectIndex, 
	FDFObjectBounds DFObjectBounds, 
	float4 TileTranslatedBoundingSphere, 
	float3 TileConeAxis,
	float TileConeAngleCos,
	float TileConeAngleSin,
	float MaxOcclusionDistance)
{
	const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PrimaryView.PreViewTranslation);
	BRANCH
	if (SphereIntersectSphere(float4(TranslatedBoundsCenter, DFObjectBounds.SphereRadius + MaxOcclusionDistance), TileTranslatedBoundingSphere)
#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
		// This 'can object cast on tile bounds' test has to be disabled because the weight used to combine occlusion from multiple objects is computed based on distance to occluder only
		//&& SphereIntersectConeWithMaxDistance(float4(SphereCenterAndRadius.xyz, SphereCenterAndRadius.w + TileBoundingSphere.w), TileBoundingSphere.xyz, TileConeAxis, TileConeAngleCos, TileConeAngleSin, MaxOcclusionDistance)
#endif
		)
	{
#if USE_DISTANCE_FIELD_FOR_TILE_CULLING
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
		float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);
		float3 VolumeSamplePosition = mul(float4(TileTranslatedBoundingSphere.xyz, 1), TranslatedWorldToVolume).xyz;
		float3 ClampedSamplePosition = clamp(VolumeSamplePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
		float DistanceToClamped = length(ClampedSamplePosition - VolumeSamplePosition);
		float WorldDistanceToOccluder = (DistanceToMeshSurfaceStandalone(ClampedSamplePosition, DFObjectData) + DistanceToClamped) * DFObjectData.VolumeScale;

		float ErrorTolerance = 1.1f;
		// The tile can only be affected by the object's shadow if the closest part of the object is less than MaxOcclusionDistance from the tile bounds
		return WorldDistanceToOccluder - TileTranslatedBoundingSphere.w < MaxOcclusionDistance * ErrorTolerance;
#else
		return true;
#endif
	}

	return false;
}

uint CullDistanceFieldCastersToTile(
	uint ThreadIndex,
	uint GroupIndex, 
	float MaxOcclusionDistance,
	FTileCullingData TileCullingData0,
	FTileCullingData TileCullingData1)
{
#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL

	float3 ConeAxis0 = TileCullingData0.ConeAxis;
	float ConeAngleCos0 = TileCullingData0.ConeAngleCos;
	float ConeAngleSin0 = TileCullingData0.ConeAngleSin;
	float3 ConeAxis1 = TileCullingData1.ConeAxis;
	float ConeAngleCos1 = TileCullingData1.ConeAngleCos;
	float ConeAngleSin1 = TileCullingData1.ConeAngleSin;

#else

	float3 ConeAxis0 = 0;
	float ConeAngleCos0 = 0;
	float ConeAngleSin0 = 0;
	float3 ConeAxis1 = 0;
	float ConeAngleCos1 = 0;
	float ConeAngleSin1 = 0;

#endif

	LOOP
	for (uint ListObjectIndex = ThreadIndex; ListObjectIndex < NumMeshDistanceFieldCasters; ListObjectIndex += THREADGROUP_SIZEX * THREADGROUP_SIZEY)
	{
		uint ObjectIndex = MeshDistanceFieldCasterIndices[ListObjectIndex];

		#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE

			float LightAngle;
			float Unused;
			GetLightDirectionData(ListObjectIndex, true, ConeAxis0, LightAngle, Unused);
			
			ConeAngleCos0 = cos(LightAngle);
			ConeAngleSin0 = sin(LightAngle);

			ConeAxis1 = ConeAxis0;
			ConeAngleCos1 = ConeAngleCos0;
			ConeAngleSin1 = ConeAngleSin0;

		#endif

		FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
		float EffectiveMaxOcclusionDistance = MaxOcclusionDistance + .5f * DFObjectBounds.SphereRadius;

		BRANCH
		if (TileBoundsIntersectDistanceFieldCaster(ObjectIndex, DFObjectBounds, TileCullingData0.TranslatedBoundingSphere, ConeAxis0, ConeAngleCos0, ConeAngleSin0, EffectiveMaxOcclusionDistance))
		{
			uint ListIndex;
			InterlockedAdd(TileNumDistanceFields0, 1U, ListIndex);
			// Don't overwrite on overflow
			ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_DISTANCE_FIELDS - 1));
			IntersectingMeshDistanceFieldIndices[MAX_INTERSECTING_DISTANCE_FIELDS * 0 + ListIndex] = ListObjectIndex; 
		}

		BRANCH
		if (TileBoundsIntersectDistanceFieldCaster(ObjectIndex, DFObjectBounds, TileCullingData1.TranslatedBoundingSphere, ConeAxis1, ConeAngleCos1, ConeAngleSin1, EffectiveMaxOcclusionDistance))
		{
			uint ListIndex;
			InterlockedAdd(TileNumDistanceFields1, 1U, ListIndex);
			// Don't write out of bounds on overflow
			ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_DISTANCE_FIELDS - 1));
			IntersectingMeshDistanceFieldIndices[MAX_INTERSECTING_DISTANCE_FIELDS * 1 + ListIndex] = ListObjectIndex;
		}
	}

	GroupMemoryBarrierWithGroupSync();

	return min(GroupIndex == 0 ? TileNumDistanceFields0 : TileNumDistanceFields1, (uint)MAX_INTERSECTING_DISTANCE_FIELDS);
}

float ShadowConeTraceAgainstCulledDistanceFieldCasters(
	float3 TranslatedWorldRayStart,
	float3 UnitRayDirection,
	float LightAngle,  
	float MaxOcclusionDistance,
	uint CulledDataParameter, 
	uint NumIntersectingCasters,
	uniform bool bUseCulling,
	inout uint NumTraceSteps)
{
	float TanConeAngle = tan(LightAngle);

	float GeometricMeanNumerator = 0;
	float TotalWeight = 0;

	LOOP
	for (uint TileCulledObjectIndex = 0; TileCulledObjectIndex < NumIntersectingCasters; TileCulledObjectIndex++)
	{
		uint ListObjectIndex = TileCulledObjectIndex;

		if (bUseCulling)
		{
			uint GroupIndex = CulledDataParameter;
			ListObjectIndex = IntersectingMeshDistanceFieldIndices[MAX_INTERSECTING_DISTANCE_FIELDS * GroupIndex + TileCulledObjectIndex];
		}

		float MinVisibility = 0.0f;

		#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE
			GetLightDirectionData(ListObjectIndex, true, UnitRayDirection, LightAngle, MinVisibility);
			TanConeAngle = tan(LightAngle);
		#endif

		uint ObjectIndex = MeshDistanceFieldCasterIndices[ListObjectIndex];

		FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
		float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);

		// Increase max occlusion distance based on object size for distance field casters
		// This improves the solidness of the shadows, since the fadeout distance causes internal structure of objects to become visible
		float EffectiveMaxOcclusionDistance = MaxOcclusionDistance + .5f * DFObjectBounds.SphereRadius;

		float MaxSphereRadius = TanConeAngle * EffectiveMaxOcclusionDistance;
		float3 TranslatedWorldRayEnd = TranslatedWorldRayStart + UnitRayDirection * EffectiveMaxOcclusionDistance;

		float3 VolumeRayStart = mul(float4(TranslatedWorldRayStart, 1), TranslatedWorldToVolume).xyz;
		float3 VolumeRayEnd = mul(float4(TranslatedWorldRayEnd, 1), TranslatedWorldToVolume).xyz;
		float3 VolumeRayDirection = VolumeRayEnd - VolumeRayStart;
		float VolumeRayLength = length(VolumeRayDirection);
		VolumeRayDirection /= VolumeRayLength;
		float VolumeMaxSphereRadius = MaxSphereRadius / DFObjectData.VolumeScale;

		{
			FDFAssetData DFAssetData = LoadDFAssetDataHighestResolution(DFObjectData.AssetIndex);

			const float MaxEncodedDistance = DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y;

			// Prevent incorrect shadowing when sampling invalid bricks by limiting VolumeMaxSphereRadius to MaxEncodedDistance
			VolumeMaxSphereRadius = min(VolumeMaxSphereRadius, MaxEncodedDistance);

			float MinTraceVisibility = 1;
			//@todo - derive from texel size
			float StartOffset = .02f;
			uint MaxSteps = 32;
			float MinStepSize = 1.0f / (4 * MaxSteps);
			// How much to artificially slow down the stepping proportional to cone occlusion
			// Reduces artifacts when steps are large (far from the surface) yet heavily occluded because the cone angle is large
			float FullVisibilityMaxStepFraction = .1f;
			float OcclusionExponent = .8f;

			float SampleRayTime = StartOffset;
			uint StepIndex = 0;

			LOOP
			for (; StepIndex < MaxSteps; StepIndex++)
			{
				float3 SampleVolumePosition = VolumeRayStart + VolumeRayDirection * SampleRayTime;
				float3 ClampedSamplePosition = clamp(SampleVolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
				float DistanceToClamped = length(ClampedSamplePosition - SampleVolumePosition);
				float VolumeDistanceToOccluder = SampleSparseMeshSignedDistanceField(ClampedSamplePosition, DFAssetData) + DistanceToClamped;

				float SphereRadius = clamp(TanConeAngle * SampleRayTime, 0, VolumeMaxSphereRadius);
				float StepVisibility = pow(saturate(VolumeDistanceToOccluder / SphereRadius), OcclusionExponent);

				float OccluderDistanceFraction = (SampleRayTime + VolumeDistanceToOccluder) * DFObjectData.VolumeScale / EffectiveMaxOcclusionDistance;
	
				// Fade out occlusion based on distance to occluder to avoid a discontinuity at the max AO distance
				//@todo - this introduces banding artifacts because we may be taking large steps through the fade region
				StepVisibility = max(StepVisibility, saturate(OccluderDistanceFraction));

				MinTraceVisibility = min(MinTraceVisibility, StepVisibility);

				float StepDistance = min(VolumeDistanceToOccluder, StepVisibility * FullVisibilityMaxStepFraction * VolumeRayLength);

				StepDistance = max(StepDistance, MinStepSize);
				SampleRayTime += StepDistance;
				NumTraceSteps++;

				// Terminate the trace if we reached a negative area or went past the end of the ray
				if (VolumeDistanceToOccluder <= 0 
					|| SampleRayTime > VolumeRayLength)
				{
					break;
				}
			}

			#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_PUNCTUAL

				// Attempt to match the effect of MinVisibility on capsule shadows, which combine with multiply so a MinVisibility of .1 does not actually achieve values >= .1
				MinVisibility *= MinVisibility;

				// Apply to indirect shadows only
				MinTraceVisibility = lerp(MinVisibility, 1, MinTraceVisibility);
			#endif

			float WeightDistance = EffectiveMaxOcclusionDistance;

			{
				float3 SampleVolumePosition = VolumeRayStart;
				float3 ClampedSamplePosition = clamp(SampleVolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
				float DistanceToClamped = length(ClampedSamplePosition - SampleVolumePosition);
				float VolumeDistanceToOccluder = SampleSparseMeshSignedDistanceField(ClampedSamplePosition, DFAssetData) + DistanceToClamped;

				WeightDistance = VolumeDistanceToOccluder * DFObjectData.VolumeScale;
				//WeightDistance = length(SphereCenterAndRadius.xyz - WorldRayStart);
			}

			float Weight = 1 - saturate(WeightDistance / EffectiveMaxOcclusionDistance);
			Weight = pow(Weight, 4);

			// Weighted geometric mean to combine shadows from multiple casters without over-darkening like a simple multiply would do
			GeometricMeanNumerator += Weight * log2(MinTraceVisibility);
			TotalWeight += Weight;
		}
	}

	float ConeVisibility = 1;

	if (TotalWeight > 0)
	{
		ConeVisibility = exp2(GeometricMeanNumerator / TotalWeight);
	}

	return ConeVisibility;
}

void ApplySelfShadowingIntensityForDeferred(float2 ScreenUV, inout float Visibility, float2 SVPos)
{
	BRANCH
	if (View.IndirectCapsuleSelfShadowingIntensity < 1)
	{
#if SUBSTRATE_GBUFFER_FORMAT==1
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(SVPos.xy, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		const bool bHasDynamicIndirectShadowCasterRepresentation = SubstratePixelHeader.HasDynamicIndirectShadowCasterRepresentation();
#else
		#if SHADING_PATH_MOBILE
			FGBufferData GBufferData = MobileFetchAndDecodeGBuffer(ScreenUV, SVPos);;
		#else
			FGBufferData GBufferData = GetGBufferData(ScreenUV);
		#endif
		const bool bHasDynamicIndirectShadowCasterRepresentation = HasDynamicIndirectShadowCasterRepresentation(GBufferData);
#endif
		// Reduce self shadowing intensity
		Visibility = lerp(1, Visibility, bHasDynamicIndirectShadowCasterRepresentation ? View.IndirectCapsuleSelfShadowingIntensity : 1);
	}
}

#if APPLY_TO_BENT_NORMAL
Texture2D ReceiverBentNormalTexture;
RWTexture2D<float4> RWBentNormalTexture;
#endif

uint EyeIndex;

float IndirectCapsuleSelfShadowingIntensity;
uint DownsampleFactor;
RWTexture2D<float2> RWShadowFactors;
float MaxOcclusionDistance;

uint2 TileDimensions;
RWStructuredBuffer<uint> RWTileIntersectionCounts;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void CapsuleShadowingCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint ThreadIndex = GroupThreadId.y * THREADGROUP_SIZEX + GroupThreadId.x;

	float2 ScreenUV = float2((DispatchThreadId.xy * DownsampleFactor + ScissorRectMinAndSize.xy + .5f) * View.BufferSizeAndInvSize.zw);
	float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

// Mobile does not support bent normals
#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_RECEIVER && !SHADING_PATH_MOBILE
	float4 ReceiverTextureValue = ReceiverBentNormalTexture.Load(DispatchThreadId.xyz);
	float3 ReceiverBentNormal = ReceiverTextureValue.xyz;
	float SceneDepth = ReceiverTextureValue.w;
#else
#if SHADING_PATH_MOBILE
	float SceneDepth = CalcSceneDepth(ScreenUV, EyeIndex);
#else
	float SceneDepth = CalcSceneDepth(ScreenUV);
#endif
#endif
	
	const float3 OpaqueTranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	uint CulledDataParameter = 0;
	bool bTileShouldComputeShadowing = true;
	FTileCullingData TileCullingData0;
	FTileCullingData TileCullingData1;
	uint NumPixelIntersectingShapes = 0;
	uint NumTileIntersectingShapes = 0;
	uint NumDistanceFieldSteps = 0;

	// So we can skip skybox pixels / tiles without having to check the GBuffer for shading model
	float MaxDepth = 20000;

#define USE_CULLING 1
#if USE_CULLING

	SetupTileCullingData(SceneDepth, MaxDepth, ThreadIndex, GroupId.xy, TileCullingData0, TileCullingData1, bTileShouldComputeShadowing, CulledDataParameter);

#endif // USE_CULLING

	float Visibility = 1;
	
	BRANCH
	if (bTileShouldComputeShadowing)
	{
		// World space offset along the start of the ray to avoid incorrect self-shadowing
		float RayStartOffset = 0;
		float LightVectorLength = 0;
		
		#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_PUNCTUAL
			#if POINT_LIGHT

				float3 LightVector = LightTranslatedPositionAndInvRadius.xyz - OpaqueTranslatedWorldPosition;
				LightVectorLength = length(LightVector);
				float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition + LightVector / LightVectorLength * RayStartOffset;
				float3 UnitRayDirection = (LightTranslatedPositionAndInvRadius.xyz - OpaqueTranslatedWorldPosition) / LightVectorLength;
				float LightAngle = atanFastPos(LightSourceRadius / LightVectorLength);

			#else

				float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition + LightDirection * RayStartOffset;
				float3 UnitRayDirection = LightDirection;
				float LightAngle = LightAngleAndNormalThreshold.x;

			#endif
		#elif LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_RECEIVER && !SHADING_PATH_MOBILE
			float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition;
			float BentNormalLength = length(ReceiverBentNormal);
			float3 UnitRayDirection = ReceiverBentNormal / max(BentNormalLength, .00001f);
			float LightAngle = max(BentNormalLength * .5f * PI, PI / 8);
		#else
			float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition;
			float3 UnitRayDirection = 0;
			float LightAngle = 0;
		#endif

		uint NumIntersectingCapsules = NumShadowCapsules;
		uint NumIntersectingDistanceFieldCasters = NumMeshDistanceFieldCasters;

		#if USE_CULLING
			#if SUPPORT_CAPSULE_SHAPES
				NumIntersectingCapsules = CullCapsuleShapesToTile(
					ThreadIndex,
					CulledDataParameter, 
					MaxOcclusionDistance,
					TileCullingData0,
					TileCullingData1);
				NumTileIntersectingShapes += TileNumCapsules0 + TileNumCapsules1;
			#endif

			#if SUPPORT_MESH_DISTANCE_FIELDS
				NumIntersectingDistanceFieldCasters = CullDistanceFieldCastersToTile(
					ThreadIndex,
					CulledDataParameter, 
					MaxOcclusionDistance,
					TileCullingData0,
					TileCullingData1);
		
				NumTileIntersectingShapes += TileNumDistanceFields0 + TileNumDistanceFields1;
			#endif
		#else
			NumTileIntersectingShapes = NumShadowCapsules + NumMeshDistanceFieldCasters;
		#endif
			
		NumPixelIntersectingShapes += NumIntersectingCapsules + NumIntersectingDistanceFieldCasters;
			
		#if SUPPORT_CAPSULE_SHAPES
			Visibility *= ShadowConeTraceAgainstCulledCapsuleShapes(
				TranslatedWorldRayStart,
				UnitRayDirection, 
				LightVectorLength,
				LightAngle, 
				1.0f / MaxOcclusionDistance,
				CulledDataParameter, 
				NumIntersectingCapsules,
				USE_CULLING ? true : false);
		#endif

		#if SUPPORT_MESH_DISTANCE_FIELDS
			Visibility *= ShadowConeTraceAgainstCulledDistanceFieldCasters(
				TranslatedWorldRayStart,
				UnitRayDirection,
				LightAngle,
				MaxOcclusionDistance,
				CulledDataParameter,
				NumIntersectingDistanceFieldCasters,
				USE_CULLING ? true : false,
				NumDistanceFieldSteps);
		#endif

#if !APPLY_TO_BENT_NORMAL
		if (all(GroupThreadId.xy == 0) && all(GroupId.xy < TileDimensions))
		{
			RWTileIntersectionCounts[GroupId.y * TileDimensions.x + GroupId.x] = NumTileIntersectingShapes;
		}
#endif
	}

	//Visibility = NumDistanceFieldSteps / 20.0f;
	//Visibility = NumPixelIntersectingShapes / 20.0f;
	//Visibility = bTileShouldComputeShadowing ? 1 : 0;

#if APPLY_TO_BENT_NORMAL
#if LIGHT_SOURCE_MODE != LIGHT_SOURCE_FROM_RECEIVER
		float3 ReceiverBentNormal = ReceiverBentNormalTexture.Load(DispatchThreadId.xyz).xyz;
#endif
	#if LIGHT_SOURCE_MODE == LIGHT_SOURCE_FROM_CAPSULE && !FORWARD_SHADING
		// The third param of this function is used only by mobile paths which do not support bent normals.
		ApplySelfShadowingIntensityForDeferred(ScreenUV, Visibility, float2(0,0));
	#endif
	#if METAL_ES3_1_PROFILE 
		// clamp max depth to avoid #inf
		SceneDepth = min(SceneDepth, 65500.0f);
	#endif
	RWBentNormalTexture[DispatchThreadId.xy] = float4(ReceiverBentNormal * Visibility, SceneDepth);
#else
	RWShadowFactors[DispatchThreadId.xy] = float2(Visibility, SceneDepth);
#endif
}

StructuredBuffer<uint> TileIntersectionCounts;

// Size of a tile in NDC
float2 TileSize;

#ifndef TILES_PER_INSTANCE
#define TILES_PER_INSTANCE 1
#endif

void CapsuleShadowingUpsampleVS(
	float2 TexCoord : ATTRIBUTE0,
	uint VertexId : SV_VertexID,
	uint InstanceId : SV_InstanceID, 
	out float4 OutPosition : SV_POSITION
	)
{
	// Compute the actual instance id for when multiple tiles are packed into the vertex buffer
	uint EffectiveInstanceId = InstanceId * TILES_PER_INSTANCE + VertexId / 4;
	uint NumCapsulesAffectingTile = TileIntersectionCounts[EffectiveInstanceId];
	uint TileY = InstanceId / TileDimensions.x;
	uint2 TileCoordinate = uint2(EffectiveInstanceId - TileY * TileDimensions.x, TileY);
	float2 ScreenUV = ((TileCoordinate + TexCoord) * TileSize + ScissorRectMinAndSize.xy) * View.BufferSizeAndInvSize.zw;
	float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	OutPosition = float4(ScreenPosition, 0, 1);

	// Cull the tile if no affecting capsules, shadow will not be visible
	if (NumCapsulesAffectingTile == 0)
	{
		OutPosition.xy = 0;
	}
}

#ifdef UPSAMPLE_PASS

Texture2D ShadowFactorsTexture;
SamplerState ShadowFactorsSampler;
float2 ShadowFactorsUVBilinearMax;

float OutputtingToLightAttenuation;

void CapsuleShadowingUpsamplePS(
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
#if APPLY_TO_SSAO
	,out float4 OutAmbientOcclusion : SV_Target1
#endif
	)
{
	const float2 ScreenUV = SvPositionToBufferUV(SVPos);

	float Output;
	float SceneDepth;
	UpsampleShadowFactors(SVPos, ScissorRectMinAndSize, 1.0f / DOWNSAMPLE_FACTOR, 0, POSITIVE_INFINITY, ShadowFactorsTexture, ShadowFactorsSampler, ShadowFactorsUVBilinearMax, Output, SceneDepth, EyeIndex);

	if (OutputtingToLightAttenuation > 0)
	{
		OutColor = EncodeLightAttenuation(Output).xxxx;
	}
	else
	{
		#if !FORWARD_SHADING
			// Self-shadowing determination is binary so apply at full res where possible
			ApplySelfShadowingIntensityForDeferred(ScreenUV, Output, SVPos.xy);
		#endif

		OutColor = Output;
	}

#if APPLY_TO_SSAO
	OutAmbientOcclusion = Output;
#endif
}

#endif // UPSAMPLE_PASS
