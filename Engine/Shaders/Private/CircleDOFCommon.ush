// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CircleDOFCommon.usf: Circle depth of field common
=============================================================================*/



// CircleDOF: Compute circle of confusion size in half res pixels.
float DepthToCoc(float SceneDepth) 
{
	// {radius, depth blur amp, depth blur radius, ViewPortScale}
	float4 CircleDofParams = View.CircleDOFParams;

	// We intentionally don't do the DepthOfFieldFocalRegion as it breaks realism.

	float Focus = View.DepthOfFieldFocalDistance;
	float Radius = CircleDofParams.x;
	float CocRadius = ((SceneDepth - Focus) / SceneDepth) * Radius;
	float DepthBlurRadius = (1.0 - exp2(-SceneDepth * CircleDofParams.y)) * CircleDofParams.z;
	float ReturnCoc = max(abs(CocRadius), DepthBlurRadius);
	if(CocRadius < 0.0) 
	{
		// near CoC is using negative values
		ReturnCoc = -ReturnCoc;
	}
	return ReturnCoc;
}
