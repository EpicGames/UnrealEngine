// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float3 GetClearCoatBottomNormal(FGBufferData GBuffer, float3 WorldNormal)
{
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
	{
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(WorldNormal);
		WorldNormal = OctahedronToUnitVector(oct1);
	}

	return WorldNormal;
}

float GetClearCoatRoughness(FGBufferData GBuffer)
{
	return GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT ? GBuffer.CustomData.y : GBuffer.Roughness;
}

void RemapClearCoatDiffuseAndSpecularColor(FGBufferData GBuffer, half NoV, inout half3 DiffuseColor, inout half3 SpecularColor)
{
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		// Attenuate base color and recompute diffuse color
		half RefractionScale = ((NoV * 0.5 + 0.5) * NoV - 1) * saturate(1.25 - 1.25 * GBuffer.Roughness) + 1;

		half MetalSpec = 0.9;
		half3 AbsorptionColor = GBuffer.BaseColor * (1 / MetalSpec);
		half3 Absorption = AbsorptionColor * ((NoV - 1) * 0.85 * (1 - lerp(AbsorptionColor, Square(AbsorptionColor), -0.78)) + 1);

		half F0 = 0.04;
		half Fc = Pow5(1 - NoV);
		half F = Fc + (1 - Fc) * F0;
		half ClearCoat = GBuffer.CustomData.x;
		half LayerAttenuation = lerp(1, (1 - F), ClearCoat);

		half3 BaseColor = lerp(GBuffer.BaseColor * LayerAttenuation, MetalSpec * Absorption * RefractionScale, GBuffer.Metallic * ClearCoat);
		//BaseColor += Dither / 255.f;
		DiffuseColor = BaseColor - BaseColor * GBuffer.Metallic;

		half Specular = lerp(GBuffer.Specular, RefractionScale, ClearCoat);
		SpecularColor = ComputeF0(Specular, BaseColor, GBuffer.Metallic) * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
	}
}

void RemapClearCoatDiffuseAndSpecularColor(FGBufferData GBuffer, float2 ScreenPosition, inout half3 DiffuseColor, inout half3 SpecularColor)
{
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), View.ScreenToTranslatedWorld).xyz;
		half3 V = -GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
		half NoV = saturate(dot(GBuffer.WorldNormal, V));

		RemapClearCoatDiffuseAndSpecularColor(GBuffer, NoV, DiffuseColor, SpecularColor);
	}
}

float3 ClearCoatLayerCombine(FGBufferData GBuffer, float NoV, float3 TopLayerReflections, float3 BottomLayerReflections, float3 SpecularColor)
{
	const float ClearCoat			= GBuffer.CustomData.x;
	const float ClearCoatRoughness	= GBuffer.CustomData.y;

	// TODO EnvBRDF should have a mask param
	float2 AB = PreIntegratedGF.SampleLevel( PreIntegratedGFSampler, float2( NoV, GBuffer.Roughness ), 0 ).rg;
	BottomLayerReflections *= SpecularColor * AB.x + AB.y * saturate( 50 * SpecularColor.g ) * (1 - ClearCoat);
		
	// F_Schlick
	float F = EnvBRDF( 0.04, ClearCoatRoughness, NoV ).x;
		
	F *= ClearCoat;
			
	return BottomLayerReflections * (1 - F) + TopLayerReflections * F;
}