// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ClearReplacement.usf: Collection of Shaders for alternative ways to clear a texture/buffer.
=============================================================================*/

#include "Common.ush"
#include "ComputeShaderUtils.ush"

#ifndef THREADGROUPSIZE_X
	#define THREADGROUPSIZE_X 1
	#define THREADGROUPSIZE_Y 1
	#define THREADGROUPSIZE_Z 1
#endif

#ifndef USE_WRAPPED_GROUP_COUNT
	#define USE_WRAPPED_GROUP_COUNT 0
#endif

#if ENABLE_CLEAR_VALUE
	VALUE_TYPE ClearValue;
#else
	static const VALUE_TYPE ClearValue = (VALUE_TYPE)0;
#endif

#ifdef RESOURCE_TYPE

	#if ENABLE_BOUNDS
		uint4 MinBounds;
		uint4 MaxBounds;
	#endif

	#if RESOURCE_TYPE == 0

		RWBuffer        <VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).x
		#define COMP(a,b) ((a).x < (b).x)

	#elif RESOURCE_TYPE == 1

		RWTexture2D     <VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).xy
		#define COMP(a,b) (((a).x < (b).x) && ((a).y < (b).y))

	#elif RESOURCE_TYPE == 2

		RWTexture2DArray<VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).xyz
		#define COMP(a,b) (((a).x < (b).x) && ((a).y < (b).y) && ((a).z < (b).z))

	#elif RESOURCE_TYPE == 3

		RWTexture3D     <VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).xyz
		#define COMP(a,b) (((a).x < (b).x) && ((a).y < (b).y) && ((a).z < (b).z))

	#elif RESOURCE_TYPE == 4
	
		RWStructuredBuffer<VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).x
		#define COMP(a,b) ((a).x < (b).x)

	#elif RESOURCE_TYPE == 5
	
		RWBuffer        <VALUE_TYPE> ClearResource;
		#define ADDR(p) (p).x
		#define COMP(a,b) ((a).x < (b).x)

	#endif

	[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
	void ClearCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex, uint3 Position : SV_DispatchThreadID)
	{
	#if USE_WRAPPED_GROUP_COUNT
		Position = uint3(GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUPSIZE_X), 0, 0);
	#endif

	#if ENABLE_BOUNDS
		ADDR(Position) += ADDR(MinBounds);
		if (COMP(Position, MaxBounds))
	#endif
		{
			ClearResource[ADDR(Position)] = ClearValue;
		}
	}

	void ClearTextureRWPS(float4 SvPosition : SV_POSITION)
	{
#if ENABLE_BOUNDS
		ADDR(SvPosition) += ADDR(MinBounds);
		if (COMP(SvPosition, MaxBounds))
#endif
		{
			ClearResource[ADDR(SvPosition)] = ClearValue;
		}
	}

#else

	#if ENABLE_BOUNDS
		float2 MinBounds;
		float2 MaxBounds;
	#endif

	#if ENABLE_DEPTH
		float Depth;
	#else
		static const float Depth = 0;
	#endif

	float4 ClearVS(uint Id : SV_VertexID) : SV_POSITION
	{
		float2 UV = float2(Id & 1, Id >> 1);

#if ENABLE_BOUNDS
		float2 Pos = lerp(MinBounds, MaxBounds, UV);
#else
		float2 Pos = UV * 2 - 1;
#endif

		return float4(Pos, Depth, 1);
	}

	VALUE_TYPE ClearPS() : SV_Target0
	{
		return ClearValue;
	}

#endif
