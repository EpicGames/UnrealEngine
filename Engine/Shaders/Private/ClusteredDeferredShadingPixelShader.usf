// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
	ClusteredDeferredShadingPixelShader - applies all local lights in the
	LightGrid in a full-screen pass, not directional ones as they are done
	in the traditional deferred passes.
=============================================================================*/

// Following the lead of Tiled Deferred 
#define SUPPORT_CONTACT_SHADOWS 0
#define USE_IES_PROFILE 1
#define NON_DIRECTIONAL_DIRECT_LIGHTING 0
#define SUBSTRATE_SSS_TRANSMISSION USE_TRANSMISSION

// Used in RectLight.ush, and requires a per-light texture which we cannot support at present.
// If the textures are atlased or something like that in the future it should be ok to turn on.
#define USE_SOURCE_TEXTURE 0

#if SUBSTRATE_ENABLED
  #if SUBSTRATE_TILETYPE == 0
	#define SUBSTRATE_FASTPATH 1
  #elif SUBSTRATE_TILETYPE == 1
	#define SUBSTRATE_SINGLEPATH 1
  #elif SUBSTRATE_TILETYPE == 2
	// COMPLEX PATH
  #elif SUBSTRATE_TILETYPE == 3
	// COMPLEX PATH
	#define SUBSTRATE_COMPLEXSPECIALPATH 1
  #else
	#error Substrate tile type non-implemented
  #endif
#endif

#include "HairStrands/HairStrandsVisibilityCommonStruct.ush" 
#include "Common.ush"
#include "Substrate/Substrate.ush"
#include "DeferredShadingCommon.ush"  
#include "BRDF.ush"
#include "ShadingModels.ush"
#include "LightGridCommon.ush"
#include "DeferredLightingCommon.ush"
#include "LightData.ush"
#include "VirtualShadowMaps/VirtualShadowMapTransmissionCommon.ush"
#include "VirtualShadowMaps/VirtualShadowMapMaskBitsCommon.ush"
#include "Substrate/SubstrateEvaluation.ush"
#include "Substrate/SubstrateDeferredLighting.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

#if USE_HAIR_LIGHTING == 1
	#include "HairStrands/HairStrandsCommon.ush"
	#include "HairStrands/HairStrandsVisibilityCommon.ush"
	#include "HairStrands/HairStrandsVisibilityUtils.ush"
	#include "HairStrands/HairStrandsDeepTransmittanceCommon.ush"
	#include "HairStrands/HairStrandsDeepTransmittanceDualScattering.ush" 
#endif // USE_HAIR_LIGHTING

// Causes the shader to run one pass over the light list per shading model, which cuts down peak register pressure.
// Moves some overhead out of the inner loop (testing for shading model) which ought to be a good thing also.
#if SUBSTRATE_ENABLED
	#define USE_PASS_PER_SHADING_MODEL 0
#else
	// Only enable on non-FXC compilers since FXC takes ages to compile all the shader model passes.
	#define USE_PASS_PER_SHADING_MODEL (COMPILER_DXC == 1 || COMPILER_PSSL == 1 || COMPILER_METAL == 1)
#endif

// Temporary work around for shader compilation issue
#define SUPPORT_SUBSTRATE_LIGHTING (COMPILER_DXC == 1 || COMPILER_PSSL == 1 || COMPILER_METAL == 1)

// Enable cluster debug visualization
#define GRID_DEBUG_ENABLE 0

Texture2D<uint4> ShadowMaskBits;

#if USE_HAIR_LIGHTING
uint				HairTransmittanceBufferMaxCount;
Buffer<uint>		HairTransmittanceBuffer;
#endif

/**
 * Debug function for visualizing clusters
 */ 
bool GetClusterDebugColor(float2 LocalPosition, float SceneDepth, FCulledLightsGridHeader GridData, inout float4 OutColor)
{
	OutColor = 0.0f;
#if GRID_DEBUG_ENABLE
	uint ZSlice = (uint)(max(0, log2(SceneDepth * GridData.LightGridZParams.x + GridData.LightGridZParams.y) * GridData.LightGridZParams.z));
	ZSlice = min(ZSlice, (uint)(GridData.CulledGridSize.z - 1));
	uint3 GridCoordinate = uint3(uint2(LocalPosition.x, LocalPosition.y) >> GridData.LightGridPixelSizeShift, ZSlice);

	if (ZSlice % 2)
	{
		OutColor = float(ZSlice) / float(GridData.CulledGridSize.z - 1);
	}
	else
	{
		OutColor = 1.0f - float(ZSlice) / float(GridData.CulledGridSize.z - 1);
	}

	if (GridCoordinate.x % 2)
	{
		OutColor.xyz = 1.0f - OutColor.xyz;
	}
	if (GridCoordinate.y % 2)
	{
		OutColor.xyz = 1.0f - OutColor.xyz;
	}
	OutColor.w = 0.0f;
	return true;
#else
	return false;
#endif
}

/**
 * Adds local lighting using the light grid, does not apply directional lights, as they are done elsewhere.
 * Does not support dynamic shadows, as these require the per-light shadow mask.
 */
float4 GetLightGridLocalLighting(
#if SUBSTRATE_GBUFFER_FORMAT==1 && USE_HAIR_LIGHTING == 0
	FSubstrateAddressing SubstrateAddressing,
	FSubstratePixelHeader SubstratePixelHeader,
	inout float3 OutOpaqueRoughRefractionSceneColor,
	inout float3 OutSubSurfaceSceneColor,
#else 
	FScreenSpaceData ScreenSpaceData, 
#endif
	const FCulledLightsGridHeader InLightGridHeader, 
	float3 TranslatedWorldPosition, 
	float3 CameraVector, 
	float2 ScreenUV,
	float SceneDepth,
	uint EyeIndex, 
	float Dither, 
	uint SampleIndex=0, 
	uint TotalSampleCount=0, 
	uint2 PixelCoord=0,
	uint HairLightChannelMask=0)
{
	float4 DirectLighting = 0;

	// Limit max to ForwardLightStruct.NumLocalLights.
	// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
	const uint NumLightsInGridCell = min(InLightGridHeader.NumLights, GetMaxLightsPerCell());

	int2 PixelPos = int2( ScreenUV.xy * View.BufferSizeAndInvSize.xy );
	uint4 ShadowMask = ShadowMaskBits[ PixelPos ];
	#if USE_HAIR_LIGHTING
	uint LightChannelMask = HairLightChannelMask;
	#else
	uint LightChannelMask = GetSceneLightingChannel(PixelPos);
	#endif
	
	LOOP
	for (uint GridLightListIndex = 0; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
	{
		const FLocalLightData LocalLight = GetLocalLightDataFromGrid(InLightGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);

		// The lights are sorted such that all that support clustered deferred are at the beginning, there might be others
		// (e.g., lights with dynamic shadows) so we break out when the condition fails.
		if (!UnpackIsClusteredDeferredSupported(LocalLight))
		{
			break;
		}

		if (!IsLightVisible(LocalLight, TranslatedWorldPosition))
		{
			continue;
		}

		if ((LightChannelMask & UnpackLightingChannelMask(LocalLight)) == 0)
		{
			continue;
		}

		FDeferredLightData LightData = ConvertToDeferredLight(LocalLight);
		LightData.ShadowedBits = 1;
		LightData.bRectLight = LightData.bRectLight && USE_RECT_LIGHT;
		#if USE_IES_PROFILE
		LightData.Color *= ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);
		#endif
		float4 LightAttenuation = float4(1, 1, 1, 1);

		// Find packed VSM projection result if present
		// NOTE: We have to do the search as in the deferred pass again currently as the shadow masks
		// are packed according to a prune light list that contains only shadow-casting lights.
		uint VSMGridLightListIndex = GridLightListIndex;
		if (LocalLight.Internal.VirtualShadowMapId != INDEX_NONE)
		{
			LightAttenuation = GetVirtualShadowMapMaskForLight(
				ShadowMask,
				uint2(PixelPos),
				SceneDepth,
				LocalLight.Internal.VirtualShadowMapId,
				TranslatedWorldPosition,
				VSMGridLightListIndex).xxxx;
		}

		#if USE_HAIR_LIGHTING
		if (LightAttenuation.x > 0)
		{
			const float3 L = normalize(LightData.TranslatedWorldPosition - TranslatedWorldPosition);
			const float3 V = normalize(-CameraVector);

			// Fetch precomputed transmittance
			// * Transmittance data are only computed for shadowed lights with VSM
			// * Transmittance mask index is computed using VSM light index remapping. This needs to match HairStrandsDeepTransmittanceMask.usf
			FHairTransmittanceMask TransmittanceMask = InitHairTransmittanceMask();
			if (LocalLight.Internal.VirtualShadowMapId != INDEX_NONE && VSMGridLightListIndex < GetPackedShadowMaskMaxLightCount())
			{
				const uint TransmittanceSampleIndex = SampleIndex + VSMGridLightListIndex * TotalSampleCount;
				const uint TransmittanceSampleMaxCount = TotalSampleCount * GetPackedShadowMaskMaxLightCount();
				if (TransmittanceSampleIndex < TransmittanceSampleMaxCount)
				{
					TransmittanceMask = UnpackTransmittanceMask(HairTransmittanceBuffer[TransmittanceSampleIndex]);		
				}
			}

			// Apply dual scattering
			LightData.HairTransmittance = GetHairTransmittance(
				V,
				L,
				ScreenSpaceData.GBuffer,
				TransmittanceMask,
				View.HairScatteringLUTTexture,
				HairScatteringLUTSampler,
				View.HairComponents);

			// This shouldn't be needed any more as the virtual shadow map should offer enough spatial precision
			LightAttenuation = min(LightAttenuation, LightData.HairTransmittance.OpaqueVisibility.xxxx);
		}
		#endif

	#if SUBSTRATE_GBUFFER_FORMAT==1 && USE_HAIR_LIGHTING == 0

		FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters	= GetInitialisedSubstrateShadowTermInputParameters();
		SubstrateShadowTermInputParameters.bEvaluateShadowTerm					= true;
		SubstrateShadowTermInputParameters.SceneDepth							= CalcSceneDepth(ScreenUV);
		SubstrateShadowTermInputParameters.PrecomputedShadowFactors				= SubstrateReadPrecomputedShadowFactors(SubstratePixelHeader, PixelPos, SceneTexturesStruct.GBufferETexture);
		SubstrateShadowTermInputParameters.TranslatedWorldPosition				= TranslatedWorldPosition;
		SubstrateShadowTermInputParameters.LightAttenuation						= LightAttenuation;
		SubstrateShadowTermInputParameters.Dither								= Dither;

		const float3 V = -CameraVector;
		float3 ToLight, L;
		const float LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

		FSubstrateDeferredLighting SubstrateLighting = (FSubstrateDeferredLighting)0;
	#if SUPPORT_SUBSTRATE_LIGHTING
		SubstrateLighting = SubstrateDeferredLighting(
			LightData,
			V,
			L,
			ToLight,
			LightMask,
			SubstrateShadowTermInputParameters,
			Substrate.MaterialTextureArray,
			SubstrateAddressing,
			SubstratePixelHeader);
	#endif

		DirectLighting += SubstrateLighting.SceneColor;
	#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		OutOpaqueRoughRefractionSceneColor += SubstrateLighting.OpaqueRoughRefractionSceneColor;
		OutSubSurfaceSceneColor += SubstrateLighting.SubSurfaceSceneColor;
	#endif

	#elif 1
		// NOTE: uses AO data like tiled deferred (as opposed to forward path)
		// TODO: Passing in 0,0 as SVPos, what is meaning of this? Only used if 'REFERENCE_QUALITY' is on.
		float SurfaceShadow = 1.0f;
		DirectLighting += GetDynamicLighting(TranslatedWorldPosition, CameraVector, ScreenSpaceData.GBuffer, ScreenSpaceData.AmbientOcclusion, LightData,LightAttenuation, Dither, uint2(0, 0), SurfaceShadow);
	#else // 1
		FSimpleDeferredLightData SimpleLightData = (FSimpleDeferredLightData)0;
		SimpleLightData.TranslatedWorldPosition = LightData.TranslatedWorldPosition;
		SimpleLightData.InvRadius = LightData.InvRadius;
		SimpleLightData.Color = LightData.Color;
		SimpleLightData.FalloffExponent = LightData.FalloffExponent;
		SimpleLightData.bInverseSquared = LightData.bInverseSquared;
		DirectLighting += GetSimpleDynamicLighting(TranslatedWorldPosition, CameraVector, ScreenSpaceData.GBuffer.WorldNormal, ScreenSpaceData.AmbientOcclusion, ScreenSpaceData.GBuffer.DiffuseColor, ScreenSpaceData.GBuffer.SpecularColor, ScreenSpaceData.GBuffer.Roughness, SimpleLightData);
	#endif // SUBSTRATE_ENABLED
	}

	// For debugging
	//DirectLighting = InLightGridData.NumLights / (float)ForwardLightStruct.MaxCulledLightsPerCell;
	return DirectLighting;
}

#if USE_PASS_PER_SHADING_MODEL

// NOTE: this is a macro since we want to be sure the 'ScreenSpaceData.GBuffer.ShadingModelID = ShadingModelId' is compile time constant
//       The rest sort of follows.
// NOTE 2: The screen-space data is loaded inside the if to try to minimize register pressure, but not clear if it is respected.
#define GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(ShadingModelId, PixelShadingModelId, /*inout float4*/ CompositedLighting, ScreenUV, LightGridHeader, Dither) \
{	\
	BRANCH	\
	if (PixelShadingModelId == ShadingModelId)	\
	{	\
		float2 ScreenPosition = UVAndScreenPos.zw;	\
		FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(ScreenUV);	\
		float SceneDepth = CalcSceneDepth(ScreenUV);	\
		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;	\
		float3 CameraVector = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);	\
		ScreenSpaceData.GBuffer.ShadingModelID = ShadingModelId;	\
		CompositedLighting += GetLightGridLocalLighting(ScreenSpaceData, LightGridHeader, TranslatedWorldPosition, CameraVector, ScreenUV, SceneDepth, 0, Dither); \
	}	\
}

#endif // USE_PASS_PER_SHADING_MODEL

#if SUBSTRATE_GBUFFER_FORMAT==1 && USE_HAIR_LIGHTING == 0
void ClusteredShadingPixelShader(
	float2 ScreenUV : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	, out float3 OutOpaqueRoughRefractionSceneColor : SV_Target1
	, out float3 OutSubSurfaceSceneColor : SV_Target2
#elif SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	, out float OutSeparatedSubsurfaceDiffuse : SV_Target1
#endif
	)
{
	uint2 PixelPos = SvPosition.xy;

	const float Dither = InterleavedGradientNoise(SvPosition.xy, View.StateFrameIndexMod8);
	const uint EyeIndex = 0;
	const float SceneDepth = CalcSceneDepth(ScreenUV);		
	const float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;

	const uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SceneDepth, EyeIndex);
	const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);

	// Debug output
	#if GRID_DEBUG_ENABLE
	if (GetClusterDebugColor(LocalPosition, SceneDepth, CulledLightGridHeader, OutColor)) return;
	#endif

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);

	// NOTE: this early out helps with the case where there are no lights in the grid cell.
	if (CulledLightGridHeader.NumLights == 0 || !SubstratePixelHeader.IsSubstrateMaterial())
	{
		OutColor = 0;
		return;
	}
	
	float4 CompositedLighting = 0;
	float3 OpaqueRoughRefractionSceneColor = 0;
	float3 SubSurfaceSceneColor = 0;

	BRANCH
	if (SubstratePixelHeader.ClosureCount > 0 || CulledLightGridHeader.NumLights == 0)
	{
		// Substrate tile are expressed as buffer dimension rather than view dimension
		const float2 ViewportScreenUV = ScreenUV * View.BufferSizeAndInvSize.xy * View.ViewSizeAndInvSize.zw;
		const float2 ScreenPosition = ViewportUVToScreenPos(ViewportScreenUV);
		const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
		const float3 CameraVector = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);

		// Regular lights
		CompositedLighting += GetLightGridLocalLighting(SubstrateAddressing, SubstratePixelHeader, OpaqueRoughRefractionSceneColor, SubSurfaceSceneColor,
			CulledLightGridHeader, TranslatedWorldPosition, CameraVector, ScreenUV, SceneDepth, EyeIndex, Dither);
	}
	
#if !VISUALIZE_LIGHT_CULLING
	CompositedLighting *= View.PreExposure;
#endif
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutOpaqueRoughRefractionSceneColor = OpaqueRoughRefractionSceneColor * View.PreExposure;
	OutSubSurfaceSceneColor = SubSurfaceSceneColor * View.PreExposure;
#endif
	OutColor = CompositedLighting;
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && !SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif
}
#elif !USE_HAIR_LIGHTING
void ClusteredShadingPixelShader(
	in noperspective float4 UVAndScreenPos : TEXCOORD0,
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
, out float OutSeparatedSubsurfaceDiffuse : SV_Target1
#endif
)
{
	float2 ScreenUV = UVAndScreenPos.xy;
	float2 ScreenPosition = UVAndScreenPos.zw;

	uint PixelShadingModelID = GetScreenSpaceData(ScreenUV).GBuffer.ShadingModelID;

	const float Dither = InterleavedGradientNoise(SvPosition.xy, View.StateFrameIndexMod8);

	// ?
	const uint EyeIndex = 0;

	float SceneDepth = CalcSceneDepth(ScreenUV);
	float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;
	uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SceneDepth, EyeIndex);
	const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);

	// Debug output
	#if GRID_DEBUG_ENABLE
	if (GetClusterDebugColor(LocalPosition, SceneDepth, CulledLightGridHeader, OutColor)) return;
	#endif

	// NOTE: this early out helps with the case where there are no lights in the grid cell.
	if (CulledLightGridHeader.NumLights == 0 || PixelShadingModelID == SHADINGMODELID_UNLIT)
	{
		OutColor = 0;
		return;
	}

	float4 CompositedLighting = 0;
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	float3 CameraVector = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
	uint FirstNonSimpleLightIndex = 0;

	// Regular lights
#if USE_PASS_PER_SHADING_MODEL

	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_DEFAULT_LIT,			PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_SUBSURFACE,			PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_PREINTEGRATED_SKIN,	PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_CLEAR_COAT,			PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_SUBSURFACE_PROFILE,	PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_TWOSIDED_FOLIAGE,	PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_HAIR,				PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_CLOTH,				PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_EYE,					PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	GET_LIGHT_GRID_LOCAL_LIGHTING_SINGLE_SM(SHADINGMODELID_SINGLELAYERWATER,	PixelShadingModelID, CompositedLighting, ScreenUV, CulledLightGridHeader, Dither);
	// SHADINGMODELID_THIN_TRANSLUCENT - skipping because it can not be opaque
#else // !USE_PASS_PER_SHADING_MODEL
	CompositedLighting += GetLightGridLocalLighting(GetScreenSpaceData(ScreenUV), CulledLightGridHeader, TranslatedWorldPosition, CameraVector, ScreenUV, SceneDepth, 0, Dither);
#endif // USE_PASS_PER_SHADING_MODEL
	
#if !VISUALIZE_LIGHT_CULLING
	CompositedLighting *= View.PreExposure;
#endif
	OutColor = CompositedLighting;
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif
}
#else // USE_HAIR_LIGHTING
void ClusteredShadingPixelShader(
	in float4 SvPosition: SV_Position,
	nointerpolation in uint TotalSampleCount : DISPATCH_NODECOUNT,
	nointerpolation in uint2 Resolution : DISPATCH_RESOLUTION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	const uint EyeIndex = 0;

	const uint2 InCoord = uint2(SvPosition.xy);
	const uint SampleIndex = InCoord.x + InCoord.y * Resolution.x;
	if (SampleIndex >= TotalSampleCount)
	{
		return;
	}

	const uint2 PixelCoord = HairStrands.HairSampleCoords[SampleIndex];
	const float2 ScreenUV  = (PixelCoord + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	const float2 ScreenPosition = (ScreenUV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	const FPackedHairSample PackedSample = HairStrands.HairSampleData[SampleIndex];
	const FHairSample Sample = UnpackHairSample(PackedSample);

	const float SceneDepth = ConvertFromDeviceZ(Sample.Depth);
	
	const float2 LocalPosition = PixelCoord - View.ViewRectMin.xy;
	FScreenSpaceData ScreenSpaceData = (FScreenSpaceData)0;
	{
		ScreenSpaceData.AmbientOcclusion = 1;
		ScreenSpaceData.GBuffer = HairSampleToGBufferData(Sample, HairStrands.HairDualScatteringRoughnessOverride);
	}

	const float Dither = InterleavedGradientNoise(SvPosition.xy, View.StateFrameIndexMod8);

	const uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SceneDepth, EyeIndex);
	const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);
	// NOTE: this early out helps with the case where there are no lights in the grid cell.
	if (CulledLightGridHeader.NumLights == 0)
	{		
		return;
	}

	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 CameraVector = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);

	float4 CompositedLighting = 0;
	CompositedLighting += GetLightGridLocalLighting(ScreenSpaceData, CulledLightGridHeader, TranslatedWorldPosition, CameraVector, ScreenUV, SceneDepth, 0, Dither, SampleIndex, TotalSampleCount, PixelCoord, Sample.LightChannelMask);
#if !VISUALIZE_LIGHT_CULLING
	CompositedLighting *= View.PreExposure;
#endif

	// Weight hair sample with its coverage
	const float LocalCoverage = From8bitCoverage(Sample.Coverage8bit);
	OutColor.rgb = CompositedLighting.xyz * LocalCoverage;
	OutColor.a = LocalCoverage;
}
#endif // USE_HAIR_LIGHTING

