// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ClusteredDeferredShadingVertexShader.usf: 
=============================================================================*/

#include "HairStrands/HairStrandsVisibilityCommonStruct.ush" 
#include "Common.ush"
#include "HairStrands/HairStrandsVisibilityCommon.ush"

struct FHairStrandsSamplesDesc
{
	nointerpolation uint NodeCount : DISPATCH_NODECOUNT;
	nointerpolation uint2 Resolution : DISPATCH_RESOLUTION;
};

// This vertex shader is only used with hair strands lighting
void ClusteredShadingVertexShader(
	in uint InVertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	nointerpolation out uint OutNodeCount : DISPATCH_NODECOUNT,
	nointerpolation out uint2 OutResolution : DISPATCH_RESOLUTION)
{
	OutNodeCount = HairStrands.HairSampleCount[0];
	OutResolution = GetHairSampleResolution(OutNodeCount);

	const float2 ClipCoord = ((OutResolution + 0.5f) / float2(HairStrands.HairSampleViewportResolution)) * 2;

	const float2 UV = float2(InVertexId & 1, InVertexId >> 1);
	const float2 Pos = float2(-UV.x, UV.y) * 4 + float2(-1, +1) + float2(ClipCoord.x, -ClipCoord.y);
	OutPosition = float4(Pos, 0.5f, 1);
}
