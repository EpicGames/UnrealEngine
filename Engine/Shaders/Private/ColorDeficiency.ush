// Copyright Epic Games, Inc. All Rights Reserved.

// CDV tables comes from:
// A Model for Simulation of Color Vision Deficiency and A Color Contrast Enhancement Technique for Dichromats
// - Gustavo M. Machado (2010)

static const float3 RGB_2_Protanomaly[11 * 3] =
{
	// Severity - 0
	float3(1.0, 0.0, 0.0),
	float3(0.0, 1.0, 0.0),
	float3(0.0, 0.0, 1.0),
	// Severity - 1
	float3(0.856167, 0.182038, -0.038205),
	float3(0.029342, 0.955115, 0.015544),
	float3(-0.002880, -0.001563, 1.004443),
	// Severity - 2
	float3(0.734766, 0.334872, -0.069637),
	float3(0.051840, 0.919198, 0.028963),
	float3(-0.004928, -0.004209, 1.009137),
	// Severity - 3
	float3(0.630323, 0.465641, -0.095964),
	float3(0.069181, 0.890046, 0.040773),
	float3(-0.006308, -0.007724, 1.014032),
	// Severity - 4
	float3(0.539009, 0.579343, -0.118352),
	float3(0.082546, 0.866121, 0.051332),
	float3(-0.007136, -0.011959, 1.019095),
	// Severity - 5
	float3(0.458064, 0.679578, -0.137642),
	float3(0.092785, 0.846313, 0.060902),
	float3(-0.007494, -0.016807, 1.024301),
	// Severity - 6
	float3(0.385450, 0.769005, -0.154455),
	float3(0.100526, 0.829802, 0.069673),
	float3(-0.007442, -0.022190, 1.029632),
	// Severity - 7
	float3(0.319627, 0.849633, -0.169261),
	float3(0.106241, 0.815969, 0.077790),
	float3(-0.007025, -0.028051, 1.035076),
	// Severity - 8
	float3(0.259411, 0.923008, -0.182420),
	float3(0.110296, 0.804340, 0.085364),
	float3(-0.006276, -0.034346, 1.040622),
	// Severity - 9
	float3(0.203876, 0.990338, -0.194214),
	float3(0.112975, 0.794542, 0.092483),
	float3(-0.005222, -0.041043, 1.046265),
	// Severity - 10
	float3(0.152286, 1.052583, -0.204868),
	float3(0.114503, 0.786281, 0.099216),
	float3(-0.003882, -0.048116, 1.051998)
};

static const float3 RGB_2_Deuteranomaly[11 * 3] =
{
	// Severity - 0
	float3(1.0, 0.0, 0.0),
	float3(0.0, 1.0, 0.0),
	float3(0.0, 0.0, 1.0),
	// Severity - 1
	float3(0.866435, 0.177704, -0.044139),
	float3(0.049567, 0.939063, 0.011370),
	float3(-0.003453, 0.007233, 0.996220),
	// Severity - 2
	float3(0.760729, 0.319078, -0.079807),
	float3(0.090568, 0.889315, 0.020117),
	float3(-0.006027, 0.013325, 0.992702),
	// Severity - 3
	float3(0.675425, 0.433850, -0.109275),
	float3(0.125303, 0.847755, 0.026942),
	float3(-0.007950, 0.018572, 0.989378),
	// Severity - 4
	float3(0.605511, 0.528560, -0.134071),
	float3(0.155318, 0.812366, 0.032316),
	float3(-0.009376, 0.023176, 0.986200),
	// Severity - 5
	float3(0.547494, 0.607765, -0.155259),
	float3(0.181692, 0.781742, 0.036566),
	float3(-0.010410, 0.027275, 0.983136),
	// Severity - 6
	float3(0.498864, 0.674741, -0.173604),
	float3(0.205199, 0.754872, 0.039929),
	float3(-0.011131, 0.030969, 0.980162),
	// Severity - 7
	float3(0.457771, 0.731899, -0.189670),
	float3(0.226409, 0.731012, 0.042579),
	float3(-0.011595, 0.034333, 0.977261),
	// Severity - 8
	float3(0.422823, 0.781057, -0.203881),
	float3(0.245752, 0.709602, 0.044646),
	float3(-0.011843, 0.037423, 0.974421),
	// Severity - 9
	float3(0.392952, 0.823610, -0.216562),
	float3(0.263559, 0.690210, 0.046232),
	float3(-0.011910, 0.040281, 0.971630),
	// Severity - 10
	float3(0.367322, 0.860646, -0.227968),
	float3(0.280085, 0.672501, 0.047413),
	float3(-0.011820, 0.042940, 0.968881)
};

static const float3 RGB_2_Tritanomaly[11 * 3] =
{
	// Severity - 0
	float3(1.0, 0.0, 0.0),
	float3(0.0, 1.0, 0.0),
	float3(0.0, 0.0, 1.0),
	// Severity - 1
	float3(0.926670, 0.092514, -0.019184),
	float3(0.021191, 0.964503, 0.014306),
	float3(0.008437, 0.054813, 0.936750),
	// Severity - 2
	float3(0.895720, 0.133330, -0.029050),
	float3(0.029997, 0.945400, 0.024603),
	float3(0.013027, 0.104707, 0.882266),
	// Severity - 3
	float3(0.905871, 0.127791, -0.033662),
	float3(0.026856, 0.941251, 0.031893),
	float3(0.013410, 0.148296, 0.838294),
	// Severity - 4
	float3(0.948035, 0.089490, -0.037526),
	float3(0.014364, 0.946792, 0.038844),
	float3(0.010853, 0.193991, 0.795156),
	// Severity - 5
	float3(1.017277, 0.027029, -0.044306),
	float3(-0.006113, 0.958479, 0.047634),
	float3(0.006379, 0.248708, 0.744913),
	// Severity - 6
	float3(1.104996, -0.046633, -0.058363),
	float3(-0.032137, 0.971635, 0.060503),
	float3(0.001336, 0.317922, 0.680742),
	// Severity - 7
	float3(1.193214, -0.109812, -0.083402),
	float3(-0.058496, 0.979410, 0.079086),
	float3(-0.002346, 0.403492, 0.598854),
	// Severity - 8
	float3(1.257728, -0.139648, -0.118081),
	float3(-0.078003, 0.975409, 0.102594),
	float3(-0.003316, 0.501214, 0.502102),
	// Severity - 9
	float3(1.278864, -0.125333, -0.153531),
	float3(-0.084748, 0.957674, 0.127074),
	float3(-0.000989, 0.601151, 0.399838),
	// Severity - 10
	float3(1.255528, -0.076749, -0.178779),
	float3(-0.078411, 0.930809, 0.147602),
	float3(0.004733, 0.691367, 0.303900)
};

#define CDV_NormalVision 0
#define CDV_Deuteranope 1
#define CDV_Protanope 2
#define CDV_Tritanope 3


float3 ConvertSourceRGBToDeficientRGB(float3 SourceRGB, float ColorVisionDeficiencySeverity, float ColorVisionDeficiencyType)
{
	float3 DeficientRGB = SourceRGB;
	int Index = (int)ColorVisionDeficiencySeverity * 3;
	if (ColorVisionDeficiencyType == CDV_Deuteranope)
	{
		float3x3 Mat = float3x3(RGB_2_Deuteranomaly[Index], RGB_2_Deuteranomaly[Index + 1], RGB_2_Deuteranomaly[Index + 2]);
		DeficientRGB = mul(Mat, SourceRGB);
	}
	else if (ColorVisionDeficiencyType == CDV_Protanope)
	{
		float3x3 Mat = float3x3(RGB_2_Protanomaly[Index], RGB_2_Protanomaly[Index + 1], RGB_2_Protanomaly[Index + 2]);
		DeficientRGB = mul(Mat, SourceRGB);
	}
	else if (ColorVisionDeficiencyType == CDV_Tritanope)
	{
		float3x3 Mat = float3x3(RGB_2_Tritanomaly[Index], RGB_2_Tritanomaly[Index + 1], RGB_2_Tritanomaly[Index + 2]);
		DeficientRGB = mul(Mat, SourceRGB);
	}

	return DeficientRGB;
}

// http://www.daltonize.org
static const float3x3 RGB_DALTONIZE_MAT =
{
	0.0, 0.7, 0.7,
	0.0, 1.0, 0.0,
	0.0, 0.0, 1.0,
};

static const float3x3 RGB_DALTONIZE_RED_BLIND =
{
	0.5, 1.0, 1.0,
	0.5, 0.5, 0.5,
	0.5, 0.5, 0.5,
};

static const float3x3 RGB_DALTONIZE_GREEN_BLIND =
{
	0.5, 0.0, 0.5,
	1.0, 0.0, 1.0,
	0.5, 0.0, 0.5,
};

static const float3x3 RGB_DALTONIZE_BLUE_BLIND =
{
	0.5, 0.5, 0.5,
	0.5, 0.5, 0.5,
	1.0, 1.0, 0.5,
};

#define USE_RGB_SPACE_CDV_CORRECTION 1
#define USE_SINGLE_DALTONIZE_MATRIX 0

float3 ColorDeficiency(float3 SourceRGB, float ColorVisionDeficiencyType, float ColorVisionDeficiencySeverity, float bCorrectDeficiency, float bSimulateCorrectionWithDeficiency)
{
	float3 DeficientRGB = ConvertSourceRGBToDeficientRGB(SourceRGB, ColorVisionDeficiencySeverity, ColorVisionDeficiencyType);

	float3 OutRGB = DeficientRGB;

	if (bCorrectDeficiency == 1)
	{
		#if USE_RGB_SPACE_CDV_CORRECTION
			// 
			float3 ErrorRGB = (SourceRGB - DeficientRGB);

			#if USE_SINGLE_DALTONIZE_MATRIX
				// 
				float3 CorrectionRGB = mul(RGB_DALTONIZE_MAT, ErrorRGB);
			#else
				float3 CorrectionRGB = float3(0.0, 0.0, 0.0);
				if (ColorVisionDeficiencyType == CDV_Deuteranope)
				{
					CorrectionRGB = mul(RGB_DALTONIZE_GREEN_BLIND, ErrorRGB);
				}
				else if (ColorVisionDeficiencyType == CDV_Protanope)
				{
					CorrectionRGB = mul(RGB_DALTONIZE_RED_BLIND, ErrorRGB);
				}
				else if (ColorVisionDeficiencyType == CDV_Tritanope)
				{
					CorrectionRGB = mul(RGB_DALTONIZE_BLUE_BLIND, ErrorRGB);
				}
			#endif

			// 
			OutRGB = SourceRGB + CorrectionRGB;
		#endif

		if (bSimulateCorrectionWithDeficiency == 1)
		{
			OutRGB = ConvertSourceRGBToDeficientRGB(OutRGB, ColorVisionDeficiencySeverity, ColorVisionDeficiencyType);
		}
	}

	return OutRGB;
}
