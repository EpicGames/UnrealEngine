// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorSpace.ush"

//----------------------------------------------------------------------------------------------------
// reference: https://en.wikipedia.org/wiki/Color_difference
// CIE76
float DeltaE_CIE76(float3 Lab0, float3 Lab1)
{
	return length(Lab0 - Lab1);
}

// graphics arts
static float CIE_k_L = 1;
static float CIE_K_1 = 0.045;
static float CIE_K_2 = 0.015;

static float CIE_k_C = 1;
static float CIE_k_H = 1;

float DeltaE_CIE94(float3 Lab0, float3 Lab1)
{
	float Delta_L = Lab0.x - Lab1.x;
	float C_1 = length(Lab0.yz);
	float C_2 = length(Lab1.yz);
	float Delta_C_ab = C_1 - C_2;

	float2 Delta_ab = Lab0.yz - Lab1.yz;
	float Delta_H_ab_sqr = length2(Delta_ab) - 2 * Delta_C_ab * Delta_C_ab;

	float S_L = 1;
	float S_C = 1 + CIE_K_1 * C_1;
	float S_H = 1 + CIE_K_2 * C_2;

	return sqrt(
		Delta_L * Delta_L / ((CIE_k_L * S_L) * (CIE_k_L * S_L)) +
		Delta_C_ab * Delta_C_ab / ((CIE_k_C * S_C) * (CIE_k_C * S_C)) +
		Delta_H_ab_sqr / ((CIE_k_H * S_H) * (CIE_k_H * S_H))
	);
}

float CosDeg(float Degree)
{
	return cos(Degree * PI / 180.0f);
}

float SinDeg(float Degree)
{
	return sin(Degree * PI / 180.0f);
}

// Verification is done with data in table 1
// from "The CIEDE2000 Color-Difference
// Formula: Implementation Notes,
// Supplementary Test Data, and
// Mathematical Observations"
// http://www2.ece.rochester.edu/~gsharma/ciede2000/ciede2000noteCRNA.pdf
float DeltaE_CIE2000(float3 Lab0, float3 Lab1)
{
	float3 Lch0 = LAB_2_LCH(Lab0);
	float3 Lch1 = LAB_2_LCH(Lab1);

	float Delta_L = Lab1.x - Lab0.x;
	float L_bar = (Lab1.x + Lab0.x) * 0.5f;
	float C_bar = (Lch0.y + Lch1.y) * 0.5f;

	float component = sqrt(pow(C_bar, 7) / (pow(C_bar, 7) + pow(25, 7)));
	float G = 0.5f * (1.0f - component);
	float a_1 = Lab0.y * (1.0f + G);
	float a_2 = Lab1.y * (1.0f + G);

	float C = (Lch0.y + Lch1.y) * 0.5f;

	float3 Lch0_ap = LAB_2_LCH(float3(Lab0.x, a_1, Lab0.z));
	float3 Lch1_ap = LAB_2_LCH(float3(Lab1.x, a_2, Lab1.z));
	float C_bar_ap = (Lch0_ap.y + Lch1_ap.y) * 0.5f;
	float Delta_C_ap = Lch1_ap.y - Lch0_ap.y;

	float h1_ap = Lch0_ap.z;
	float h2_ap = Lch1_ap.z;

	float Abs_h2_ap_minus_h1_ap = abs(h1_ap - h2_ap);
	float Delta_h_ap = (Abs_h2_ap_minus_h1_ap <= 180) ? (h2_ap - h1_ap) : (
		(Abs_h2_ap_minus_h1_ap > 180 && h2_ap <= h1_ap) ? (h2_ap - h1_ap + 360) :
		(h2_ap - h1_ap - 360)
		);
	// Corner case: Set Delta_h_ap to zero if either C is zero.
	if (Lch0_ap.y == 0 || Lch1_ap.y == 0)
	{
		Delta_h_ap = 0.0f;
	}

	// Delta H' and H'
	float Delta_H_ap = 2 * sqrt(Lch0_ap.y * Lch1_ap.y) * SinDeg(Delta_h_ap * 0.5f);
	float h1_ap_plus_h2_ap = h1_ap + h2_ap;
	float H_bar_ap = (Abs_h2_ap_minus_h1_ap <= 180) ? (h1_ap_plus_h2_ap * 0.5f) : (
		(Abs_h2_ap_minus_h1_ap > 180 && h1_ap_plus_h2_ap < 360) ? ((h1_ap_plus_h2_ap + 360) * 0.5f) :
		((h1_ap_plus_h2_ap - 360) * 0.5f)
		);

	// Corner case: When either C'_1 or C'_2 is zero, H_bar' = h'_1 + h'_2;
	if (Lch0_ap.y == 0 || Lch1_ap.y == 0)
	{
		H_bar_ap = h1_ap_plus_h2_ap;
	}

	// Set T
	float T = 1 - 0.17f * CosDeg(H_bar_ap - 30.0f) + 0.24f * CosDeg(2.0f * H_bar_ap)
		+ 0.32f * CosDeg(3.0f * H_bar_ap + 6.0f) - 0.20f * CosDeg(4.0f * H_bar_ap - 63.0f);

	float L_bar_minus_50_power2 = (L_bar - 50.0f) * (L_bar - 50.0f);
	float S_L = 1 + 0.015f * L_bar_minus_50_power2 / sqrt(20.0f + L_bar_minus_50_power2);
	float S_C = 1 + 0.045f * C_bar_ap;
	float S_H = 1 + 0.015f * C_bar_ap * T;
	float component_bar = sqrt(pow(C_bar_ap, 7) / (pow(C_bar_ap, 7) + pow(25, 7)));
	float R_T = -2.0f * component_bar * SinDeg(60.0f * exp(-pow((H_bar_ap - 275.0f) / 25.0f, 2.0f)));

	float C_component = Delta_C_ap / (CIE_k_C * S_C);
	float H_component = Delta_H_ap / (CIE_k_H * S_H);
	float Delta_E_Star_00 = sqrt(
		pow(Delta_L / (CIE_k_L * S_L), 2.0f) +
		pow(C_component, 2.0f) +
		pow(H_component, 2.0f) +
		R_T * C_component * H_component
	);

	return Delta_E_Star_00;
}
