// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	ColorMap.ush: Generate matplotlib style gradient color maps for scalars in range [0..1]
    Original code: https://www.shadertoy.com/view/3lBXR3 by Matt Zucker    
===============================================================================================*/

#pragma once

float3 ColorMapViridis(float t)
{
    const float3 c0 = float3(0.2777273272234177, 0.005407344544966578, 0.3340998053353061);
    const float3 c1 = float3(0.1050930431085774, 1.404613529898575, 1.384590162594685);
    const float3 c2 = float3(-0.3308618287255563, 0.214847559468213, 0.09509516302823659);
    const float3 c3 = float3(-4.634230498983486, -5.799100973351585, -19.33244095627987);
    const float3 c4 = float3(6.228269936347081, 14.17993336680509, 56.69055260068105);
    const float3 c5 = float3(4.776384997670288, -13.74514537774601, -65.35303263337234);
    const float3 c6 = float3(-5.435455855934631, 4.645852612178535, 26.3124352495832);

    t = saturate(t);
    return c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6)))));
}

float3 ColorMapPlasma(float t)
{
    const float3 c0 = float3(0.05873234392399702, 0.02333670892565664, 0.5433401826748754);
    const float3 c1 = float3(2.176514634195958, 0.2383834171260182, 0.7539604599784036);
    const float3 c2 = float3(-2.689460476458034, -7.455851135738909, 3.110799939717086);
    const float3 c3 = float3(6.130348345893603, 42.3461881477227, -28.51885465332158);
    const float3 c4 = float3(-11.10743619062271, -82.66631109428045, 60.13984767418263);
    const float3 c5 = float3(10.02306557647065, 71.41361770095349, -54.07218655560067);
    const float3 c6 = float3(-3.658713842777788, -22.93153465461149, 18.19190778539828);

    t = saturate(t);
    return c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6)))));
}

float3 ColorMapMagma(float t)
{
    const float3 c0 = float3(-0.002136485053939582, -0.000749655052795221, -0.005386127855323933);
    const float3 c1 = float3(0.2516605407371642, 0.6775232436837668, 2.494026599312351);
    const float3 c2 = float3(8.353717279216625, -3.577719514958484, 0.3144679030132573);
    const float3 c3 = float3(-27.66873308576866, 14.26473078096533, -13.64921318813922);
    const float3 c4 = float3(52.17613981234068, -27.94360607168351, 12.94416944238394);
    const float3 c5 = float3(-50.76852536473588, 29.04658282127291, 4.23415299384598);
    const float3 c6 = float3(18.65570506591883, -11.48977351997711, -5.601961508734096);

    t = saturate(t);
    return c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6)))));
}

float3 ColorMapInferno(float t)
{
    const float3 c0 = float3(0.0002189403691192265, 0.001651004631001012, -0.01948089843709184);
    const float3 c1 = float3(0.1065134194856116, 0.5639564367884091, 3.932712388889277);
    const float3 c2 = float3(11.60249308247187, -3.972853965665698, -15.9423941062914);
    const float3 c3 = float3(-41.70399613139459, 17.43639888205313, 44.35414519872813);
    const float3 c4 = float3(77.162935699427, -33.40235894210092, -81.80730925738993);
    const float3 c5 = float3(-71.31942824499214, 32.62606426397723, 73.20951985803202);
    const float3 c6 = float3(25.13112622477341, -12.24266895238567, -23.07032500287172);

    t = saturate(t);
    return c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6)))));
}

// High-contrast, but not perceptually-linear rainbow color map
// https://ai.googleblog.com/2019/08/turbo-improved-rainbow-colormap-for.html
float3 ColorMapTurbo(float t)
{
    const float3 c0 = float3(0.1140890109226559, 0.06288340699912215, 0.2248337216805064);
    const float3 c1 = float3(6.716419496985708, 3.182286745507602, 7.571581586103393);
    const float3 c2 = float3(-66.09402360453038, -4.9279827041226, -10.09439367561635);
    const float3 c3 = float3(228.7660791526501, 25.04986699771073, -91.54105330182436);
    const float3 c4 = float3(-334.8351565777451, -69.31749712757485, 288.5858850615712);
    const float3 c5 = float3(218.7637218434795, 67.52150567819112, -305.2045772184957);
    const float3 c6 = float3(-52.88903478218835, -21.54527364654712, 110.5174647748972);

    t = saturate(t);
    return c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6)))));
}

/*=============================================================================================
	Additional color maps (not coming from the ShaderToy mentioned above)
===============================================================================================*/

#include "ColorSpace.ush"  

float3 GetHSVDebugColor(float x)
{
	float c = (1 - saturate(x)) * 0.6; // Remap [0,1] to Blue-Red
	return x > 0 ? HSV_2_LinearRGB(float3(c, 1, 1)) : float3(0, 0, 0);
}
