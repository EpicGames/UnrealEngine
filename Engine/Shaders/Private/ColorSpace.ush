// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------------------
// LMS
// https://en.wikipedia.org/wiki/LMS_color_space
//------------------------------------------------------------------

static const float3x3 sRGB_2_LMS_MAT =
{
    17.8824,  43.5161,  4.1193,
     3.4557,  27.1554,  3.8671,
     0.02996,  0.18431, 1.4670,
};

static const float3x3 LMS_2_sRGB_MAT =
{
    0.0809,  -0.1305,   0.1167,
   -0.0102,   0.0540,  -0.1136,
   -0.0003,  -0.0041,   0.6935,
};

float3 sRGB_2_LMS( float3 RGB )
{
	return mul(sRGB_2_LMS_MAT, RGB);
}

float3 LMS_2_sRGB( float3 LMS )
{
	return mul(LMS_2_sRGB_MAT, LMS);
}

//------------------------------------------------------------------
// CIE XYZ
// https://en.wikipedia.org/wiki/CIE_1931_color_space
//------------------------------------------------------------------

static const float3x3 XYZ_2_Linear_sRGB_MAT =
{
	 3.2409699419, -1.5373831776, -0.4986107603,
	-0.9692436363,  1.8759675015,  0.0415550574,
	 0.0556300797, -0.2039769589,  1.0569715142,
};

static const float3x3 Linear_sRGB_2_XYZ_MAT =
{
	0.4123907993, 0.3575843394, 0.1804807884,
	0.2126390059, 0.7151686788, 0.0721923154,
	0.0193308187, 0.1191947798, 0.9505321522,
};

float3 LinearRGB_2_XYZ( float3 LinearRGB )
{
#if WORKING_COLOR_SPACE_IS_SRGB
	return mul(Linear_sRGB_2_XYZ_MAT, LinearRGB);
#else
	return mul(WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT, LinearRGB);
#endif
}

float3 XYZ_2_LinearRGB( float3 XYZ )
{
#if WORKING_COLOR_SPACE_IS_SRGB
	return mul(XYZ_2_Linear_sRGB_MAT, XYZ);
#else
	return mul(XYZ_TO_RGB_WORKING_COLOR_SPACE_MAT, XYZ);
#endif
}

//------------------------------------------------------------------
// CIE LAB
// https://en.wikipedia.org/wiki/Lab_color_space
//------------------------------------------------------------------
// Assume XYZ is in nominal range to reduce the operation on a constant coefficient 100.
// E.g., white RGB (1, 1, 1) maps to XYZ (0.9504559271, 1.0, 1.0890577508)
// for a reference white of D65. LAB is still in the original range.

#if WORKING_COLOR_SPACE_IS_SRGB
static const float3 XYZ_WHITE_REF_NOMINAL = float3(0.9504559271, 1.0, 1.0890577508);
#else
static const float3 XYZ_WHITE_REF_NOMINAL = mul(WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT, float3(1.0, 1.0, 1.0));
#endif

static const float XYZ_2_LAB_DELTA_SQUARED = 0.04280618311; // (6/29)^2
static const float XYZ_2_LAB_DELTA_CUBED = 0.00885645167; // (6/29)^3

float xyz_otherwise(float t)
{
	return (t / (3.0 * XYZ_2_LAB_DELTA_SQUARED)) + 4.0 / 29.0;
}

float3 LinearRGB_2_LAB( float3 LinearRGB )
{
	float3 XYZ = LinearRGB_2_XYZ(LinearRGB);

	float t_X = XYZ.x / XYZ_WHITE_REF_NOMINAL.x;
	float t_Y = XYZ.y / XYZ_WHITE_REF_NOMINAL.y;
	float t_Z = XYZ.z / XYZ_WHITE_REF_NOMINAL.z;

	float f_X = (t_X > XYZ_2_LAB_DELTA_CUBED) ? pow(max(t_X, XYZ_2_LAB_DELTA_CUBED), 1.0 / 3.0) : xyz_otherwise(t_X);
	float f_Y = (t_Y > XYZ_2_LAB_DELTA_CUBED) ? pow(max(t_Y, XYZ_2_LAB_DELTA_CUBED), 1.0 / 3.0) : xyz_otherwise(t_Y);
	float f_Z = (t_Z > XYZ_2_LAB_DELTA_CUBED) ? pow(max(t_Z, XYZ_2_LAB_DELTA_CUBED), 1.0 / 3.0) : xyz_otherwise(t_Z);

	float L = ( 116.0 * f_Y ) - 16.0;
	float a = 500.0 * ( f_X - f_Y );
	float b = 200.0 * ( f_Y - f_Z );

	return float3(L, a, b);
}

float lab_otherwise(float t)
{
	return (3.0 * XYZ_2_LAB_DELTA_SQUARED) * (t - (4.0 / 29.0));
}

float3 LAB_2_LinearRGB( float3 LAB )
{
	float L = LAB.x;
	float a = LAB.y;
	float b = LAB.z;

	float t_y = (L + 16.0) / 116.0;
	float t_x = t_y + (a / 500.0);
	float t_z = t_y - (b / 200.0);

	float f_x = pow(t_x, 3.0);
	float f_y = pow(t_y, 3.0);
	float f_z = pow(t_z, 3.0);

	if (f_x <= XYZ_2_LAB_DELTA_CUBED)
	{
		f_x = lab_otherwise(t_x);
	}

	if (f_y <= XYZ_2_LAB_DELTA_CUBED)
	{
		f_y = lab_otherwise(t_y);
	}

	if (f_z <= XYZ_2_LAB_DELTA_CUBED)
	{
		f_z = lab_otherwise(t_z);
	}

	float X = XYZ_WHITE_REF_NOMINAL.x * f_x;
	float Y = XYZ_WHITE_REF_NOMINAL.y * f_y;
	float Z = XYZ_WHITE_REF_NOMINAL.z * f_z;

	return XYZ_2_LinearRGB(float3(X, Y, Z));
}

//------------------------------------------------------------------
// YCoCg and LCoCg color space for luma / chroma orthogonal processing
//------------------------------------------------------------------

// Multiplier that get applied for ALU optimisation purposes on the luma, when converting LinearRGB to YCoCg.
#define LINEARRGB_2_YCOCG_LUMA_MULTIPLIER 4.0

// YCoCg ranges: 
//  - Y  in [0, LINEARRGB_2_YCOCG_LUMA_MULTIPLIER]
//  - Co in [-2, 2]
//  - Cg in [-2, 2]

float3 LinearRGB_2_YCoCg(float3 RGB)
{
	float Y = dot(RGB, float3(1, 2, 1));
	float Co = dot(RGB, float3(2, 0, -2));
	float Cg = dot(RGB, float3(-1, 2, -1));

	float3 YCoCg = float3(Y, Co, Cg);
	return YCoCg;
}

float3 YCoCg_2_LinearRGB(float3 YCoCg)
{
	float Y = YCoCg.x * 0.25;
	float Co = YCoCg.y * 0.25;
	float Cg = YCoCg.z * 0.25;

	float R = Y + Co - Cg;
	float G = Y + Cg;
	float B = Y - Co - Cg;

	float3 RGB = float3(R, G, B);
	return RGB;
}

float3 YCoCg_2_LCoCg(float3 YCoCg)
{
	return float3(
		YCoCg.x,
		YCoCg.yz * (YCoCg.x > 0 ? rcp(YCoCg.x) : 0));
}

float3 LCoCg_2_YCoCg(float3 LCoCg)
{
	return float3(LCoCg.x, LCoCg.x * LCoCg.yz);
}

float3 LinearRGB_2_LCoCg(float3 RGB)
{
	return YCoCg_2_LCoCg(LinearRGB_2_YCoCg(RGB));
}

float3 LCoCg_2_LinearRGB(float3 LCoCg)
{
	return YCoCg_2_LinearRGB(LCoCg_2_YCoCg(LCoCg));
}

// NormalisedYCoCg has Y, Co and Cg values mapped to [0, 1]
float3 LinearRGB_2_NormalisedYCoCg(float3 RGB)
{
	return LinearRGB_2_YCoCg(RGB) * float3(1.0f / LINEARRGB_2_YCOCG_LUMA_MULTIPLIER, 0.25f, 0.25f) + float3(0.0f, 0.5f, 0.5f);
}

float3 NormalisedYCoCg_2_LinearRGB(float3 YCoCg)
{
	return YCoCg_2_LinearRGB(YCoCg * float3(LINEARRGB_2_YCOCG_LUMA_MULTIPLIER, 4.0f, 4.0f) + float3(0.0f, -2.0f, -2.0f));
}

//------------------------------------------------------------------
// HSV
//------------------------------------------------------------------

float3 HUE_2_LinearRGB(in float H)
{
	float R = abs(H * 6 - 3) - 1;
	float G = 2 - abs(H * 6 - 2);
	float B = 2 - abs(H * 6 - 4);
	return saturate(float3(R, G, B));
}

float3 HSV_2_LinearRGB(in float3 HSV)
{
	float3 RGB = HUE_2_LinearRGB(HSV.x);
	return ((RGB - 1) * HSV.y + 1) * HSV.z;
}

float3 RGB_2_HCV(in float3 RGB)
{
	// Based on work by Sam Hocevar and Emil Persson
	float4 P = (RGB.g < RGB.b)	? float4(RGB.bg, -1.0f, 2.0f / 3.0f): float4(RGB.gb, 0.0f, -1.0f / 3.0f);
	float4 Q = (RGB.r < P.x)	? float4(P.xyw, RGB.r)				: float4(RGB.r, P.yzx);
	float Chroma = Q.x - min(Q.w, Q.y);
	float Hue = abs((Q.w - Q.y) / (6.0f * Chroma + 1e-10f) + Q.z);
	return float3(Hue, Chroma, Q.x);
}

float3 LinearRGB_2_HSV(in float3 RGB)
{
	float3 HCV = RGB_2_HCV(RGB);
	float s = HCV.y / (HCV.z + 1e-10f);
	return float3(HCV.x, s, HCV.z);
}

// adapted from https://www.shadertoy.com/view/MsS3Wc by Inigo Quilez
float3 HSV_2_LinearRGB_Smooth(float3 HSV)
{
	float3 RGB = clamp(abs((HSV.x * 6.0 + float3(0.0, 4.0, 2.0)) % 6.0 - 3.0) - 1.0, 0.0, 1.0);
	RGB = RGB * RGB * (3.0 - 2.0 * RGB); // cubic smoothing	
	return HSV.z * lerp(float3(1.0, 1.0, 1.0), RGB, HSV.y);
}

//------------------------------------------------------------------
// LCH
//------------------------------------------------------------------

float3 LAB_2_LCH(float3 LAB)
{
	float3 LCH;
	LCH.x = LAB.x;
	LCH.y = length(LAB.yz);
	float HInDegree0To360 = 0.0f;

	if (LAB.z != 0 || LAB.y != 0)
	{
		HInDegree0To360 = atan2(LAB.z, LAB.y) * 180 / 3.1415926535897932f;
		HInDegree0To360 += lerp(0, 360.0f, HInDegree0To360 < 0);
	}
	LCH.z = HInDegree0To360;
	return LCH;
}


//------------------------------------------------------------------
// sRGB / Working Color Space
//------------------------------------------------------------------
float3 SRGBCoefficientsToWorkingColorSpace(in float3 CoefficientsSRGB)
{
#if WORKING_COLOR_SPACE_IS_SRGB
	return CoefficientsSRGB;
#else
	float3 TransmittanceSRGB = exp(-CoefficientsSRGB);
	float3 TransmittanceWCS = mul(SRGB_TO_WORKING_COLOR_SPACE_MAT, TransmittanceSRGB);
	return -log(TransmittanceWCS);
#endif
}

float3 SRGBColorToWorkingColorSpace(in float3 ColorSRGB)
{
#if WORKING_COLOR_SPACE_IS_SRGB
	return ColorSRGB;
#else
	return mul(SRGB_TO_WORKING_COLOR_SPACE_MAT, ColorSRGB);
#endif
}
