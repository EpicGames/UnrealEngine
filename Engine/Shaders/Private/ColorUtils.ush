// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GammaCorrectionCommon.ush"

/*=============================================================================
	ColorUtils.hlsl: Some utility functions for dealing with colors.
=============================================================================*/

void ReplicateChannel(inout float4 BaseColor, half4 InComponentReplicate, half4 InComponentReplicateAlpha)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	//In SM4/5, doing a texture lookup from a G8 texture gives float4(value,0,0,0), so we need to replicate it to the other channels.  
	//On all other platforms, the value is replicated automatically.
	if(any(InComponentReplicate != 0.0))
	{
		BaseColor = dot(BaseColor,InComponentReplicate);
	}
#endif
	BaseColor.a = dot(BaseColor,InComponentReplicateAlpha);
}


float3 YuvToRgb(float3 YUV, float4x4 ColorTransform, uint SrgbToLinear)
{
	float3x3 Coeff = float3x3(
		ColorTransform[0].xyz,
		ColorTransform[1].xyz,
		ColorTransform[2].xyz
	);

	// Offset in last column of matrix
	YUV -= float3(ColorTransform[0].w, ColorTransform[1].w, ColorTransform[2].w);
	float3 sRGB = mul(Coeff, YUV);

	return (SrgbToLinear == 1) ? sRGBToLinear(sRGB) : sRGB;
}


float3 RgbToYuv(float3 RGB, float4x4 ColorTransform, uint InLinearToSrgb)
{
	float3 TempRGB = RGB;
	if (InLinearToSrgb == 1)
	{
		TempRGB = LinearToSrgb(RGB);
	}

	// Offset in last column of matrix, we can then use it directly 
	// with 4x4 matrix multiplication with homogeneous rgb vector.
	float3 YUV = mul(ColorTransform, float4(TempRGB, 1.0f)).xyz;

	return float3(
		clamp(YUV.x, 0.0f, 1.0f),
		clamp(YUV.y, 0.0f, 1.0f),
		clamp(YUV.z, 0.0f, 1.0f)
		);
}