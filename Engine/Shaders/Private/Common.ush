// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PlatformCommon.usf: Common shader code
=============================================================================*/

#pragma once

#include "/Engine/Public/Platform.ush"

#include "PackUnpack.ush"

// These types are used for material translator generated code, or any functions the translated code can call
#if PIXELSHADER && !FORCE_MATERIAL_FLOAT_FULL_PRECISION
	#define MaterialFloat half
	#define MaterialFloat2 half2
	#define MaterialFloat3 half3
	#define MaterialFloat4 half4
	#define MaterialFloat3x3 half3x3
	#define MaterialFloat4x4 half4x4 
	#define MaterialFloat4x3 half4x3 
#else
	// Material translated vertex shader code always uses floats, 
	// Because it's used for things like world position and UVs
	#define MaterialFloat float
	#define MaterialFloat2 float2
	#define MaterialFloat3 float3
	#define MaterialFloat4 float4
	#define MaterialFloat3x3 float3x3
	#define MaterialFloat4x4 float4x4 
	#define MaterialFloat4x3 float4x3 
#endif

#ifndef COMPUTE_SHADED
#define COMPUTE_SHADED 0
#endif

struct FloatDeriv
{
	float Value;
	float Ddx;
	float Ddy;
};

struct FloatDeriv2
{
	float2 Value;
	float2 Ddx;
	float2 Ddy;
};

struct FloatDeriv3
{
	float3 Value;
	float3 Ddx;
	float3 Ddy;
};

struct FloatDeriv4
{
	float4 Value;
	float4 Ddx;
	float4 Ddy;
};

FloatDeriv ConstructFloatDeriv(float InValue, float InDdx, float InDdy)
{
	FloatDeriv Ret;
	Ret.Value = InValue;
	Ret.Ddx = InDdx;
	Ret.Ddy = InDdy;
	return Ret;
}

FloatDeriv2 ConstructFloatDeriv2(float2 InValue, float2 InDdx, float2 InDdy)
{
	FloatDeriv2 Ret;
	Ret.Value = InValue;
	Ret.Ddx = InDdx;
	Ret.Ddy = InDdy;
	return Ret;
}

FloatDeriv3 ConstructFloatDeriv3(float3 InValue, float3 InDdx, float3 InDdy)
{
	FloatDeriv3 Ret;
	Ret.Value = InValue;
	Ret.Ddx = InDdx;
	Ret.Ddy = InDdy;
	return Ret;
}

FloatDeriv4 ConstructFloatDeriv4(float4 InValue, float4 InDdx, float4 InDdy)
{
	FloatDeriv4 Ret;
	Ret.Value = InValue;
	Ret.Ddx = InDdx;
	Ret.Ddy = InDdy;
	return Ret;
}


#define FLOAT_MAX         (asfloat(0x7F7FFFFF)) // Largest representable finite float (~3.4028235e38)
#define POSITIVE_INFINITY (asfloat(0x7F800000))
#define NEGATIVE_INFINITY (asfloat(0xFF800000))

#define METER_TO_CENTIMETER		100.0f
#define CENTIMETER_TO_METER		(1.0f / METER_TO_CENTIMETER)
#define KILOMETER_TO_METER  	1000.0f
#define METER_TO_KILOMETER  	(1.0f / KILOMETER_TO_METER)
#define KILOMETER_TO_CENTIMETER	(KILOMETER_TO_METER * METER_TO_CENTIMETER)
#define CENTIMETER_TO_KILOMETER	(1.0f / KILOMETER_TO_CENTIMETER)

#define NearDepthValue 1.0f
#define FarDepthValue  0.0f

float NearestDeviceDepth(float DepthA, 
                         float DepthB,
                         float DepthC = FarDepthValue
)
{
	return max3(DepthA, DepthB, DepthC);
}

float FarthestDeviceDepth(float DepthA, 
                          float DepthB,
                          float DepthC = NearDepthValue
)
{
	return min3(DepthA, DepthB, DepthC);
}

const static MaterialFloat PI = 3.1415926535897932f;
const static float MaxHalfFloat = 65504.0f;
const static float Max11BitsFloat = 65024.0f;
const static float Max10BitsFloat = 64512.0f;
const static float3 Max111110BitsFloat3 = float3(Max11BitsFloat, Max11BitsFloat, Max10BitsFloat);

#define SUPPORTS_TEXTURE_EXTERNAL		(COMPILER_GLSL_ES3_1)

#if !SUPPORTS_TEXTURE_EXTERNAL
#define TextureExternal Texture2D
#endif

#ifndef REGISTER
	#if COMPILER_HLSLCC
		#define REGISTER(x)
	#else
		#define REGISTER(x)		: register(x)
	#endif
#endif

#ifndef SUPPORTS_TEXTURECUBE_ARRAY
#define SUPPORTS_TEXTURECUBE_ARRAY 1
#endif

#if SUPPORTS_TEXTURECUBE_ARRAY == 0
	// Define TextureCubeArray to something which will compile so we can use it in uniform buffers
	#define TextureCubeArray TextureCube
#endif

// Control MIP level used for material texture fetches. By default only raytracing 
// shaders (i.e., !PIXELSHADER) use manual MIP level selection. A material shader 
// can opt. in to force a specific MIP level.
//
// * USE_FORCE_TEXTURE_MIP : enable/disable manual MIP level selection
// * FORCED_TEXTURE_MIP    : force a specific MIP level
//
#if COMPUTE_SHADED && !defined(USE_FORCE_TEXTURE_MIP)
	#define USE_FORCE_TEXTURE_MIP 0
#endif
#if !PIXELSHADER && !defined(USE_FORCE_TEXTURE_MIP)
	#define USE_FORCE_TEXTURE_MIP 1
#endif
#ifndef USE_FORCE_TEXTURE_MIP
	#define USE_FORCE_TEXTURE_MIP 0
#endif
#ifndef FORCED_TEXTURE_MIP
	#define FORCED_TEXTURE_MIP 0.0f
#endif

// Add definition of types used by generated uniform buffers
#include "GeneratedUniformBufferTypes.ush"

// Generated file that contains uniform buffer declarations needed by the shader being compiled 
#include "/Engine/Generated/GeneratedUniformBuffers.ush" 

// uniform buffers specifics
#include "CommonViewUniformBuffer.ush"

// In HLSL, fmod is implemented as 'Lhs - trunc(Lhs / Rhs) * Rhs'
// In some cases, using floor rather than trunc is better
float FmodFloor(float Lhs, float Rhs)
{
	return Lhs - floor(Lhs / Rhs) * Rhs;
}

float2 FmodFloor(float2 Lhs, float2 Rhs)
{
	return Lhs - floor(Lhs / Rhs) * Rhs;
}

float3 FmodFloor(float3 Lhs, float3 Rhs)
{
	return Lhs - floor(Lhs / Rhs) * Rhs;
}

float4 FmodFloor(float4 Lhs, float4 Rhs)
{
	return Lhs - floor(Lhs / Rhs) * Rhs;
}

float VectorSum(float V) { return V; }
float VectorSum(float2 V) { return V.x + V.y; }
float VectorSum(float3 V) { return V.x + V.y + V.z; }
float VectorSum(float4 V) { return V.x + V.y + V.z + V.w; }

#include "LargeWorldCoordinates.ush"

#include "InstancedStereo.ush"

#include "Definitions.usf"

#include "AssertionMacros.ush"

#ifndef VELOCITY_ENCODE_DEPTH
#define VELOCITY_ENCODE_DEPTH 1
#endif

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	#define VELOCITY_ENCODE_GAMMA 1
#else
	#define VELOCITY_ENCODE_GAMMA 0
#endif

#if OPENGL_PROFILE
	#define ENCODED_VELOCITY_TYPE uint4
#else
	#define ENCODED_VELOCITY_TYPE float4
#endif

//Tie Editor features to platform support and the COMPILE_SHADERS_FOR_DEVELOPMENT which is set via CVAR.
#define USE_EDITOR_SHADERS (PLATFORM_SUPPORTS_EDITOR_SHADERS && USE_DEVELOPMENT_SHADERS)

// Using SV_ClipDistance has overhead (15% slower base pass in triangle bound test scene on PS4) so projects have to opt-in
#define USE_GLOBAL_CLIP_PLANE (PROJECT_ALLOW_GLOBAL_CLIP_PLANE && !MATERIAL_DOMAIN_POSTPROCESS && !MATERIAL_DOMAIN_UI)

#ifndef RAYTRACINGSHADER
	#define RAYTRACINGSHADER (RAYHITGROUPSHADER || RAYMISSSHADER || RAYCALLABLESHADER)
#endif

#if RAYTRACINGSHADER

// These built-ins are not available in ray tracing
// Define dummy versions so that ray-tracing materials will at least compile
#define clip(x)
#define ddx(x) 0
#define ddy(x) 0
#define fwidth(x) 0

#endif

#ifndef USE_RAYTRACED_TEXTURE_RAYCONE_LOD
#define USE_RAYTRACED_TEXTURE_RAYCONE_LOD (RAYHITGROUPSHADER)
#endif // USE_RAYTRACED_TEXTURE_RAYCONE_LOD

static float GlobalTextureMipBias = 0;
static float GlobalRayCone_TexArea = 0;
float ComputeRayConeLod(Texture2D Tex)
{
#if USE_RAYTRACED_TEXTURE_RAYCONE_LOD
	uint2 Dimensions;
	Tex.GetDimensions(Dimensions.x, Dimensions.y);
	int TexArea = Dimensions.x * Dimensions.y;
	return 0.5f * log2(GlobalRayCone_TexArea * TexArea);
#else
    return FORCED_TEXTURE_MIP;
#endif
}

float  ClampToHalfFloatRange(float  X) { return clamp(X, float(0), MaxHalfFloat); }
float2 ClampToHalfFloatRange(float2 X) { return clamp(X, float(0).xx, MaxHalfFloat.xx); }
float3 ClampToHalfFloatRange(float3 X) { return clamp(X, float(0).xxx, MaxHalfFloat.xxx); }
float4 ClampToHalfFloatRange(float4 X) { return clamp(X, float(0).xxxx, MaxHalfFloat.xxxx); }

// This would need to be a #define in GLSL to ignore the SamplerState, however, it is currently a function call in HLSL
// for type checking of the parameters - ironically the type checking is really only needed in GLSL!
MaterialFloat4 Texture1DSample(Texture1D Tex, SamplerState Sampler, float UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 Texture2DSample(Texture2D Tex, SamplerState Sampler, float2 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, ComputeRayConeLod(Tex) + GlobalTextureMipBias);
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 Texture2DSample(Texture2D Tex, SamplerState Sampler, FloatDeriv2 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV.Value, ComputeRayConeLod(Tex) + GlobalTextureMipBias);
#else
	return Tex.SampleGrad(Sampler, UV.Value, UV.Ddx, UV.Ddy);
#endif
}
MaterialFloat Texture2DSample_A8(Texture2D Tex, SamplerState Sampler, float2 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, ComputeRayConeLod(Tex) + GlobalTextureMipBias) A8_SAMPLE_MASK;
#else
	return Tex.Sample(Sampler, UV) A8_SAMPLE_MASK;
#endif
}
MaterialFloat4 Texture3DSample(Texture3D Tex, SamplerState Sampler, float3 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 TextureCubeSample(TextureCube Tex, SamplerState Sampler, float3 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 Texture2DArraySample(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 Texture1DSampleLevel(Texture1D Tex, SamplerState Sampler, float UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat4 Texture2DSampleLevel(Texture2D Tex, SamplerState Sampler, float2 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat4 Texture2DSampleBias(Texture2D Tex, SamplerState Sampler, float2 UV, MaterialFloat MipBias)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, ComputeRayConeLod(Tex) + MipBias + GlobalTextureMipBias);
#else
	return Tex.SampleBias(Sampler, UV, MipBias);
#endif
}
MaterialFloat4 Texture2DSampleGrad(Texture2D Tex, SamplerState Sampler, float2 UV, MaterialFloat2 DDX, MaterialFloat2 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}
MaterialFloat4 Texture3DSampleLevel(Texture3D Tex, SamplerState Sampler, float3 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat4 Texture3DSampleBias(Texture3D Tex, SamplerState Sampler, float3 UV, MaterialFloat MipBias)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.SampleBias(Sampler, UV, MipBias);
#endif
}
MaterialFloat4 Texture3DSampleGrad(Texture3D Tex, SamplerState Sampler, float3 UV, MaterialFloat3 DDX, MaterialFloat3 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}
MaterialFloat4 Texture2DArraySampleLevel(Texture2DArray Tex, SamplerState Sampler, float3 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat4 TextureCubeSampleLevel(TextureCube Tex, SamplerState Sampler, float3 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat TextureCubeSampleDepthLevel(TextureCube TexDepth, SamplerState Sampler, float3 UV, MaterialFloat Mip)
{
	return TexDepth.SampleLevel(Sampler, UV, Mip).x;
}
MaterialFloat4 TextureCubeSampleBias(TextureCube Tex, SamplerState Sampler, float3 UV, MaterialFloat MipBias)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.SampleBias(Sampler, UV, MipBias);
#endif
}
MaterialFloat4 TextureCubeSampleGrad(TextureCube Tex, SamplerState Sampler, float3 UV, MaterialFloat3 DDX, MaterialFloat3 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}
MaterialFloat4 TextureExternalSample(TextureExternal Tex, SamplerState Sampler, float2 UV)
{
#if USE_FORCE_TEXTURE_MIP
	#if SUPPORTS_TEXTURE_EXTERNAL
		return Tex.SampleLevel(Sampler, UV, 0);
	#else
		return Tex.SampleLevel(Sampler, UV, ComputeRayConeLod(Tex) + GlobalTextureMipBias);
	#endif
#else
	return Tex.Sample(Sampler, UV);
#endif
}
MaterialFloat4 TextureExternalSampleGrad(TextureExternal Tex, SamplerState Sampler, float2 UV, MaterialFloat2 DDX, MaterialFloat2 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}
MaterialFloat4 TextureExternalSampleLevel(TextureExternal Tex, SamplerState Sampler, float2 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}
MaterialFloat4 Texture2DGatherRed(Texture2D Tex, SamplerState Sampler, float2 UV)
{
	return Tex.GatherRed(Sampler, UV);
}
MaterialFloat4 Texture2DGatherGreen(Texture2D Tex, SamplerState Sampler, float2 UV)
{
	return Tex.GatherGreen(Sampler, UV);
}
MaterialFloat4 Texture2DGatherBlue(Texture2D Tex, SamplerState Sampler, float2 UV)
{
	return Tex.GatherBlue(Sampler, UV);
}
MaterialFloat4 Texture2DGatherAlpha(Texture2D Tex, SamplerState Sampler, float2 UV)
{
	return Tex.GatherAlpha(Sampler, UV);
}
MaterialFloat4 TextureCubeGatherRed(TextureCube Tex, SamplerState Sampler, float3 UV)
{
	uint StatusIgnored;
	return Tex.GatherRed(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeGatherGreen(TextureCube Tex, SamplerState Sampler, float3 UV)
{
	uint StatusIgnored;
	return Tex.GatherGreen(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeGatherBlue(TextureCube Tex, SamplerState Sampler, float3 UV)
{
	uint StatusIgnored;
	return Tex.GatherBlue(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeGatherAlpha(TextureCube Tex, SamplerState Sampler, float3 UV)
{
	uint StatusIgnored;
	return Tex.GatherAlpha(Sampler, UV, StatusIgnored);
}
MaterialFloat4 Texture2DArrayGatherRed(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
	return Tex.GatherRed(Sampler, UV);
}
MaterialFloat4 Texture2DArrayGatherGreen(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
	return Tex.GatherGreen(Sampler, UV);
}
MaterialFloat4 Texture2DArrayGatherBlue(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
	return Tex.GatherBlue(Sampler, UV);
}
MaterialFloat4 Texture2DArrayGatherAlpha(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
	return Tex.GatherAlpha(Sampler, UV);
}

// Re-routed texture sampling for decals
// On iOS and Android MALI devices decal UVs has issues with deriavatives on polygon edges resulting in 'wireframe' rendering artifacts (UE-50971)
// to workaround we always sample top mip level
MaterialFloat4 Texture1DSample_Decal(Texture1D Tex, SamplerState Sampler, float UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1	
	return Texture1DSampleLevel(Tex, Sampler, UV, 0);
#else
	return Texture1DSample(Tex, Sampler, UV);
#endif
}
MaterialFloat4 Texture2DSample_Decal(Texture2D Tex, SamplerState Sampler, float2 UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1
	return Texture2DSampleLevel(Tex, Sampler, UV, 0);
#else
	return Texture2DSample(Tex, Sampler, UV);
#endif
}
MaterialFloat4 Texture3DSample_Decal(Texture3D Tex, SamplerState Sampler, float3 UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1
	return Texture3DSampleLevel(Tex, Sampler, UV, 0);
#else
	return Texture3DSample(Tex, Sampler, UV);
#endif
}
MaterialFloat4 Texture2DArraySample_Decal(Texture2DArray Tex, SamplerState Sampler, float3 UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1
	return Texture2DArraySampleLevel(Tex, Sampler, UV, 0);
#else
	return Texture2DArraySample(Tex, Sampler, UV);
#endif
}
MaterialFloat4 TextureCubeSample_Decal(TextureCube Tex, SamplerState Sampler, float3 UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1
	return TextureCubeSampleLevel(Tex, Sampler, UV, 0);
#else
	return TextureCubeSample(Tex, Sampler, UV);
#endif
}
MaterialFloat4 TextureExternalSample_Decal(TextureExternal Tex, SamplerState Sampler, float2 UV)
{
#if METAL_ES3_1_PROFILE || COMPILER_GLSL_ES3_1
	return TextureExternalSampleLevel(Tex, Sampler, UV, 0);
#else
	return TextureExternalSample(Tex, Sampler, UV);
#endif
}

MaterialFloat4 Texture2DArraySampleBias(Texture2DArray Tex, SamplerState Sampler, float3 UV, MaterialFloat MipBias)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.SampleBias(Sampler, UV, MipBias);
#endif
}
MaterialFloat4 Texture2DArraySampleGrad(Texture2DArray Tex, SamplerState Sampler, float3 UV, MaterialFloat2 DDX, MaterialFloat2 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}

//converts an input 1d to 2d position. Useful for locating z frames that have been laid out in a 2d grid like a flipbook.
float2 Tile1Dto2D(float xsize, float idx)
{
	float2 xyidx = 0;
	xyidx.y = floor(idx / xsize);
	xyidx.x = idx - xsize * xyidx.y;

	return xyidx;
}

// return a pseudovolume texture sample.
// useful for simulating 3D texturing with a 2D texture or as a texture flipbook with lerped transitions
// treats 2d layout of frames a 3d texture and performs bilinear filtering by blending with an offset Z frame.
// Wrap repeat mode along XY is not seamless. This is however enough for current sampling use cases all in [0,1].
// @param Tex          = Input Texture Object storing Volume Data
// @param inPos        = Input float3 for Position, 0-1
// @param xysize       = Input float for num frames in x,y directions
// @param numframes    = Input float for num total frames
// @param mipmode      = Sampling mode: 0 = use miplevel, 1 = use UV computed gradients, 2 = Use gradients (default=0)
// @param miplevel     = MIP level to use in mipmode=0 (default 0)
// @param InDDX, InDDY = Texture gradients in mipmode=2
float4 PseudoVolumeTexture(Texture2D Tex, SamplerState TexSampler, float3 InPos, float2 XYSize, float NumFrames,
	uint MipMode = 0, float MipLevel = 0, float2 InDDX = 0, float2 InDDY = 0)
{
	float Z = InPos.z - 0.5f / NumFrames;	// This offset is needed to have a behavior consistent with hardware sampling (voxel value is at their center)
	float ZFrame = floor(Z * NumFrames);
	float ZPhase = frac(Z * NumFrames);

	float2 UV = frac(InPos.xy) / XYSize;

	float2 CurFrame = Tile1Dto2D(XYSize.x, ZFrame) / XYSize;
	float2 NextFrame = Tile1Dto2D(XYSize.x, ZFrame + 1) / XYSize;

	float2 UVCurFrame = UV + CurFrame;
	float2 UVNextFrame = UV + NextFrame;

	float4 SampleA = 0, SampleB = 0;
	switch (MipMode)
	{
	case 0: // Mip level
		SampleA = Tex.SampleLevel(TexSampler, UVCurFrame, MipLevel);
		SampleB = Tex.SampleLevel(TexSampler, UVNextFrame, MipLevel);
		break;
	case 1: // Gradients automatic from UV
		SampleA = Texture2DSample(Tex, TexSampler, UVCurFrame);
		SampleB = Texture2DSample(Tex, TexSampler, UVNextFrame);
		break;
	case 2: // Deriviatives provided
		SampleA = Tex.SampleGrad(TexSampler, UVCurFrame,  InDDX, InDDY);
		SampleB = Tex.SampleGrad(TexSampler, UVNextFrame, InDDX, InDDY);
		break;
	default:
		break;
	}

	return lerp(SampleA, SampleB, ZPhase);
}

// return a pseudovolume texture sample where the input is a 4 channel packed texure that has neighbour slices.
// useful for simulating 3D texturing with a 2D texture or as a texture flipbook with lerped transitions
// treats 2d layout of frames a 3d texture and performs bilinear filtering by blending with an offset Z frame.
// Wrap repeat mode along XY is not seamless. This is however enough for current sampling use cases all in [0,1].
// @param Tex          = Input Texture Object storing Volume Data
// @param inPos        = Input float3 for Position, 0-1
// @param xysize       = Input float for num frames in x,y directions
// @param numframes    = Input float for num total frames
// @param mipmode      = Sampling mode: 0 = use miplevel, 1 = use UV computed gradients, 2 = Use gradients (default=0)
// @param miplevel     = MIP level to use in mipmode=0 (default 0)
// @param InDDX, InDDY = Texture gradients in mipmode=2
float2 PseudoVolumeTexture2ChannelPacked(Texture2D Tex, SamplerState TexSampler, float3 InPos, float2 XYSize, float NumFrames,
	uint MipMode = 0, float MipLevel = 0, float2 InDDX = 0, float2 InDDY = 0)
{
	float Z = InPos.z - 0.5f / NumFrames;	// This offset is needed to have a behavior consistent with hardware sampling (voxel value is at their center)
	float ZFrame = floor(Z * NumFrames);
	float ZPhase = frac(Z * NumFrames);

	float2 UV = frac(InPos.xy) / XYSize;

	float2 CurFrame = Tile1Dto2D(XYSize.x, ZFrame) / XYSize;

	float2 UVCurFrame = UV + CurFrame;

	float4 Sample = 0;
	switch (MipMode)
	{
	case 0: // Mip level
		Sample = Tex.SampleLevel(TexSampler, UVCurFrame, MipLevel);
		break;
	case 1: // Gradients automatic from UV
		Sample = Texture2DSample(Tex, TexSampler, UVCurFrame);
		break;
	case 2: // Deriviatives provided
		Sample = Tex.SampleGrad(TexSampler, UVCurFrame,  InDDX, InDDY);
		break;
	default:
		break;
	}

	return lerp(Sample.rb, Sample.ga, ZPhase);
}

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5 // Cubemap arrays are not supported in SM4 feature level
MaterialFloat4 TextureCubeArraySample(TextureCubeArray Tex, SamplerState Sampler, float4 UV)
{
	return Tex.Sample(Sampler, UV);
}

MaterialFloat4 TextureCubeArraySampleLevel(TextureCubeArray Tex, SamplerState Sampler, float4 UV, MaterialFloat Mip)
{
	return Tex.SampleLevel(Sampler, UV, Mip);
}

MaterialFloat4 TextureCubeArraySampleBias(TextureCubeArray Tex, SamplerState Sampler, float4 UV, MaterialFloat MipBias)
{
#if USE_FORCE_TEXTURE_MIP
	return Tex.SampleLevel(Sampler, UV, 0);
#else
	return Tex.SampleBias(Sampler, UV, MipBias);
#endif
}

MaterialFloat4 TextureCubeArraySampleGrad(TextureCubeArray Tex, SamplerState Sampler, float4 UV, MaterialFloat3 DDX, MaterialFloat3 DDY)
{
	return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}

// OVerload with explicit array parameter, used by some non-material shader code
MaterialFloat4 TextureCubeArraySampleLevel(TextureCubeArray Tex, SamplerState Sampler, float3 UV, float ArrayIndex, MaterialFloat Mip)
{
	return TextureCubeArraySampleLevel(Tex, Sampler, float4(UV, ArrayIndex), Mip);
}

MaterialFloat4 TextureCubeArrayGatherRed(TextureCubeArray Tex, SamplerState Sampler, float4 UV)
{
	uint StatusIgnored;
	return Tex.GatherRed(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeArrayGatherGreen(TextureCubeArray Tex, SamplerState Sampler, float4 UV)
{
	uint StatusIgnored;
	return Tex.GatherGreen(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeArrayGatherBlue(TextureCubeArray Tex, SamplerState Sampler, float4 UV)
{
	uint StatusIgnored;
	return Tex.GatherBlue(Sampler, UV, StatusIgnored);
}
MaterialFloat4 TextureCubeArrayGatherAlpha(TextureCubeArray Tex, SamplerState Sampler, float4 UV)
{
	uint StatusIgnored;
	return Tex.GatherAlpha(Sampler, UV, StatusIgnored);
}
#endif // FEATURE_LEVEL >= FEATURE_LEVEL_SM5

// TANGENTTOWORLD0 is the first row of the tangent to world matrix, w might be needed for padding and is not used yet.
// TANGENTTOWORLD2 is the last row of the tangent to world matrix, determinant of tangent basis in w
	
// Helper macro to determine whether we need to separately interpolate the world vertex normal to the pixel center.
// Currently only curvature-to-roughness needs this interpolation, so disable it when it's not used to save on interpolants.
#define USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION	(FEATURE_LEVEL >= FEATURE_LEVEL_SM5 && MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS)

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5

	#define TANGENTTOWORLD0					TEXCOORD10
	#define TANGENTTOWORLD2					TEXCOORD11

	// _centroid is needed to get better quality with MSAA

	// The D3D shader compiler combines _centroid and non centroid. Using float3 would result in a internal
	// shader compiler error. This block is using float4 to prevent that.
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	#define TANGENTTOWORLD_INTERPOLATOR_BLOCK	float4 TangentToWorld0 : TEXCOORD10_centroid; float4	TangentToWorld2	: TEXCOORD11_centroid; \
		float4	TangentToWorld2_Center	: TEXCOORD15;
#else
	#define TANGENTTOWORLD_INTERPOLATOR_BLOCK	float4 TangentToWorld0 : TEXCOORD10_centroid; float4	TangentToWorld2	: TEXCOORD11_centroid;
#endif

#else
	#define TANGENTTOWORLD0					TEXCOORD10
	#define TANGENTTOWORLD2					TEXCOORD11
	#if MOBILE_EMULATION
		#define TANGENTTOWORLD_INTERPOLATOR_BLOCK	float4 TangentToWorld0 : TANGENTTOWORLD0; float4 TangentToWorld2 : TANGENTTOWORLD2;
	#else
		#define TANGENTTOWORLD_INTERPOLATOR_BLOCK	half4 TangentToWorld0 : TANGENTTOWORLD0; half4 TangentToWorld2 : TANGENTTOWORLD2;
	#endif
#endif


float3 LuminanceFactors()
{
#if UE_LEGACY_LUMINANCE_FACTORS
	return float3(0.3, 0.59, 0.11);
#else
	#if WORKING_COLOR_SPACE_IS_SRGB || MATERIAL_DOMAIN_UI
		return float3(0.2126390059, 0.7151686788, 0.0721923154);
	#else
		return float3(WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT._m10_m11_m12);
	#endif
#endif
}

MaterialFloat Luminance( MaterialFloat3 LinearColor )
{
	return dot( LinearColor, MaterialFloat3(LuminanceFactors()));
}

float length2(float2 v)
{
	return dot(v, v);
}
float length2(float3 v)
{
	return dot(v, v);
}
float length2(float4 v)
{
	return dot(v, v);
}

uint Mod(uint a, uint b)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_ES3_1
	return a % b;
#else
	return a - (b * (uint)((float)a / (float)b));
#endif
}

uint2 Mod(uint2 a, uint2 b)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_ES3_1
	return a % b;
#else
	return a - (b * (uint2)((float2)a / (float2)b));
#endif
}

uint3 Mod(uint3 a, uint3 b)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_ES3_1
	return a % b;
#else
	return a - (b * (uint3)((float3)a / (float3)b));
#endif
}

#define POW_CLAMP 0.000001f

// Clamp the base, so it's never <= 0.0f (INF/NaN).
MaterialFloat ClampedPow(MaterialFloat X,MaterialFloat Y)
{
	return pow(max(abs(X),POW_CLAMP),Y);
}
MaterialFloat2 ClampedPow(MaterialFloat2 X,MaterialFloat2 Y)
{
	return pow(max(abs(X),MaterialFloat2(POW_CLAMP,POW_CLAMP)),Y);
}
MaterialFloat3 ClampedPow(MaterialFloat3 X,MaterialFloat3 Y)
{
	return pow(max(abs(X),MaterialFloat3(POW_CLAMP,POW_CLAMP,POW_CLAMP)),Y);
}  
MaterialFloat4 ClampedPow(MaterialFloat4 X,MaterialFloat4 Y)
{
	return pow(max(abs(X),MaterialFloat4(POW_CLAMP,POW_CLAMP,POW_CLAMP,POW_CLAMP)),Y);
} 

// Pow function that will return 0 if Base is <= 0 (or small enough to yield a floating point special).
// This is done to prevent floating point specials when compilers expands pow into exp(Exponent * log(Base)).
// Do not add the ".f" for the 2.980233e-8 because it will convert the Base from half to float, leave Exponent as half, which cause a device lost on some mali devices on Vulkan. UE-331629
MaterialFloat PositiveClampedPow(MaterialFloat Base, MaterialFloat Exponent)
{
	return (Base <= 2.980233e-8) ? 0.0f : pow(Base, Exponent);
}
MaterialFloat2 PositiveClampedPow(MaterialFloat2 Base, MaterialFloat2 Exponent)
{
	return MaterialFloat2(PositiveClampedPow(Base.x, Exponent.x), PositiveClampedPow(Base.y, Exponent.y)); 
}
MaterialFloat3 PositiveClampedPow(MaterialFloat3 Base, MaterialFloat3 Exponent)
{
	return MaterialFloat3(PositiveClampedPow(Base.xy, Exponent.xy), PositiveClampedPow(Base.z, Exponent.z)); 
}  
MaterialFloat4 PositiveClampedPow(MaterialFloat4 Base, MaterialFloat4 Exponent)
{
	return MaterialFloat4(PositiveClampedPow(Base.xy, Exponent.xy), PositiveClampedPow(Base.zw, Exponent.zw)); 
} 

float DDX(float Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddx(Input);
#endif
}

float2 DDX(float2 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddx(Input);
#endif
}

float3 DDX(float3 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddx(Input);
#endif
}

float4 DDX(float4 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddx(Input);
#endif
}

float DDY(float Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddy(Input);
#endif
}

float2 DDY(float2 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddy(Input);
#endif
}

float3 DDY(float3 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddy(Input);
#endif
}

float4 DDY(float4 Input)
{
#if USE_FORCE_TEXTURE_MIP
	return 0;
#else
	return ddy(Input);
#endif
}

#include "FastMath.ush"
#include "Random.ush"	// used by MaterialExpressionNoise
  
/** 
 * Use this function to compute the pow() in the specular computation.
 * This allows to change the implementation depending on platform or it easily can be replaced by some approxmation.
 */
MaterialFloat PhongShadingPow(MaterialFloat X, MaterialFloat Y)
{
	// The following clamping is done to prevent NaN being the result of the specular power computation.
	// Clamping has a minor performance cost.

	// In HLSL pow(a, b) is implemented as exp2(log2(a) * b).

	// For a=0 this becomes exp2(-inf * 0) = exp2(NaN) = NaN.

	// As seen in #TTP 160394 "QA Regression: PS3: Some maps have black pixelated artifacting."
	// this can cause severe image artifacts (problem was caused by specular power of 0, lightshafts propagated this to other pixels).
	// The problem appeared on PlayStation 3 but can also happen on similar PC NVidia hardware.

	// In order to avoid platform differences and rarely occuring image atrifacts we clamp the base.

	// Note: Clamping the exponent seemed to fix the issue mentioned TTP but we decided to fix the root and accept the
	// minor performance cost.

	return ClampedPow(X, Y);
}

#if FEATURE_LEVEL < FEATURE_LEVEL_ES3_1 && !COMPILER_METAL
	// DX11 (feature levels >= 10) feature sets natively supports uints in shaders; we just use floats on other platforms.
	#define uint4	int4
#endif


// Helper macro used to interpolate the given member
#define INTERPOLATE_MEMBER(member) O.member = a.member * aInterp + b.member * bInterp


#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	/** 
		* Number of MSAA samples supported by deferred passes in D3D11. 
		* This is hardcoded because it allows deferred passes to optimize for the given value (for example, unrolling a loop).
		*/
	#define NumMSAASamples 4
#endif

// shadow and light function
Texture2D		LightAttenuationTexture;
SamplerState	LightAttenuationTextureSampler;

// We don't use an inline function so we can avoid type promotion/ coercion.
#define RETURN_COLOR( Color ) ( Color )

// Convert from unorm to snorm and viceversa
float ConvertTangentUnormToSnorm8(float Input)
{
	int IntVal = int(round(Input * 255.0f));
	//negate
	IntVal = select(IntVal > 127, IntVal | 0xFFFFFF80, IntVal);
	return clamp(IntVal / 127.0f, -1, 1);
}
float2 ConvertTangentUnormToSnorm8(float2 Input)
{
	int2 IntVal = int2(round(Input * 255.0f));
	//negate
	IntVal = select(IntVal > 127, IntVal | 0xFFFFFF80, IntVal);
	return clamp(IntVal / 127.0f, -1, 1);
}
float3 ConvertTangentUnormToSnorm8(float3 Input)
{
	int3 IntVal = int3(round(Input * 255.0f));
	IntVal = select(IntVal > 127, IntVal | 0xFFFFFF80, IntVal);
	return clamp(IntVal / 127.0f, -1, 1);
}
float4 ConvertTangentUnormToSnorm8(float4 Input)
{
	int4 IntVal = int4(round(Input * 255.0f));
	//negate
	IntVal = select(IntVal > 127, IntVal | 0xFFFFFF80, IntVal);
	return clamp(IntVal / 127.0f, -1, 1);
}
float ConvertTangentUnormToSnorm16(float Input)
{
	int IntVal = int(round(Input * 65535.0f));
	//negate
	IntVal = select(IntVal > 32767, IntVal | 0xFFFF8000, IntVal);
	return clamp(IntVal / 32767.0f, -1, 1);
}
float2 ConvertTangentUnormToSnorm16(float2 Input)
{
	int2 IntVal = int2(round(Input * 65535.0f));
	//negate
	IntVal = select(IntVal > 32767, IntVal | 0xFFFF8000, IntVal);
	return clamp(IntVal / 32767.0f, -1, 1);
}
float3 ConvertTangentUnormToSnorm16(float3 Input)
{
	int3 IntVal = int3(round(Input * 65535.0f));
	IntVal = select(IntVal > 32767, IntVal | 0xFFFF8000, IntVal);
	return clamp(IntVal / 32767.0f, -1, 1);
}
float4 ConvertTangentUnormToSnorm16(float4 Input)
{
	int4 IntVal = int4(round(Input * 65535.0f));
	//negate
	IntVal = select(IntVal > 32767, IntVal | 0xFFFF8000, IntVal);
	return clamp(IntVal / 32767.0f, -1, 1);
}
float ConvertTangentSnormToUnorm8(float Input)
{
	float Res = select(Input >= 0.0f, Input * 127, ((Input + 1.0) * 127) + 128);
	return clamp(Res / 255, 0.0f, 0.99f);
}
float2 ConvertTangentSnormToUnorm8(float2 Input)
{
	float2 Res = select(Input >= 0.0f, Input * 127, ((Input + 1.0) * 127) + 128);
	return clamp(Res / 255, 0.0f, 0.99f);
}
float3 ConvertTangentSnormToUnorm8(float3 Input)
{
	float3 Res = select(Input >= 0.0f, Input * 127, ((Input + 1.0) * 127) + 128);
	return clamp(Res / 255, 0.0f, 0.99f);
}
float4 ConvertTangentSnormToUnorm8(float4 Input)
{
	float4 Res = select(Input >= 0.0f, Input * 127, ((Input + 1.0) * 127) + 128);
	return clamp(Res / 255, 0.0f, 0.99f);
}
float ConvertTangentSnormToUnorm16(float Input)
{
	float Res = select(Input >= 0.0f, Input * 32767, ((Input + 1.0) * 32767) + 32768);
	return clamp(Res / 65535, 0.0f, 0.99f);
}
float2 ConvertTangentSnormToUnorm16(float2 Input)
{
	float2 Res = select(Input >= 0.0f, Input * 32767, ((Input + 1.0) * 32767) + 32768);
	return clamp(Res / 65535, 0.0f, 0.99f);
}
float3 ConvertTangentSnormToUnorm16(float3 Input)
{
	float3 Res = select(Input >= 0.0f, Input * 32767, ((Input + 1.0) * 32767) + 32768);
	return clamp(Res / 65535, 0.0f, 0.99f);
}
float4 ConvertTangentSnormToUnorm16(float4 Input)
{
	float4 Res = select(Input >= 0.0f, Input * 32767, ((Input + 1.0) * 32767) + 32768);
	return clamp(Res / 65535, 0.0f, 0.99f);
}

// Tangent space bias/unbias
// We don't use a function so we can avoid type promotion/ coercion.
#define TangentBias(X)		(X)
#define TangentUnbias(X)	(X)

float Square( float x )
{
	return x*x;
}

float2 Square( float2 x )
{
	return x*x;
}

float3 Square( float3 x )
{
	return x*x;
}

float4 Square( float4 x )
{
	return x*x;
}

float Pow2( float x )
{
	return x*x;
}

float2 Pow2( float2 x )
{
	return x*x;
}

float3 Pow2( float3 x )
{
	return x*x;
}

float4 Pow2( float4 x )
{
	return x*x;
}

float Pow3( float x )
{
	return x*x*x;
}

float2 Pow3( float2 x )
{
	return x*x*x;
}

float3 Pow3( float3 x )
{
	return x*x*x;
}

float4 Pow3( float4 x )
{
	return x*x*x;
}

float Pow4( float x )
{
	float xx = x*x;
	return xx * xx;
}

float2 Pow4( float2 x )
{
	float2 xx = x*x;
	return xx * xx;
}

float3 Pow4( float3 x )
{
	float3 xx = x*x;
	return xx * xx;
}

float4 Pow4( float4 x )
{
	float4 xx = x*x;
	return xx * xx;
}

float Pow5( float x )
{
	float xx = x*x;
	return xx * xx * x;
}

float2 Pow5( float2 x )
{
	float2 xx = x*x;
	return xx * xx * x;
}

float3 Pow5( float3 x )
{
	float3 xx = x*x;
	return xx * xx * x;
}

float4 Pow5( float4 x )
{
	float4 xx = x*x;
	return xx * xx * x;
}

float Pow6( float x )
{
	float xx = x*x;
	return xx * xx * xx;
}

float2 Pow6( float2 x )
{
	float2 xx = x*x;
	return xx * xx * xx;
}

float3 Pow6( float3 x )
{
	float3 xx = x*x;
	return xx * xx * xx;
}

float4 Pow6( float4 x )
{
	float4 xx = x*x;
	return xx * xx * xx;
}

// Only valid for x >= 0
MaterialFloat AtanFast( MaterialFloat x )
{
	// Minimax 3 approximation
	MaterialFloat3 A = x < 1 ? MaterialFloat3( x, 0, 1 ) : MaterialFloat3( 1/x, 0.5 * PI, -1 );
	return A.y + A.z * ( ( ( -0.130234 * A.x - 0.0954105 ) * A.x + 1.00712 ) * A.x - 0.00001203333 );
}

/** Converts a linear input value into a value to be stored in the light attenuation buffer. */
MaterialFloat EncodeLightAttenuation(MaterialFloat InColor)
{
	// Apply a 1/2 power to the input, which allocates more bits for the darks and prevents banding
	// Similar to storing colors in gamma space, except this uses less instructions than a pow(x, 1/2.2)
	return sqrt(InColor);
}

/** Converts a linear input value into a value to be stored in the light attenuation buffer. */
MaterialFloat4 EncodeLightAttenuation(MaterialFloat4 InColor)
{
	return sqrt(InColor);
}

/** Converts value stored in the light attenuation buffer into a linear light attenuation value. */
MaterialFloat DecodeLightAttenuation(MaterialFloat InColor)
{
	return Square(InColor);
}

/** Converts value stored in the light attenuation buffer into a linear light attenuation value. */
MaterialFloat4 DecodeLightAttenuation(MaterialFloat4 InColor)
{
	return Square(InColor);
}

// Like RGBM but this can be interpolated.
MaterialFloat4 RGBTEncode(MaterialFloat3 Color)
{
	MaterialFloat4 RGBT;
	MaterialFloat Max = max(max(Color.r, Color.g), max(Color.b, 1e-6));
	MaterialFloat RcpMax = rcp(Max);
	RGBT.rgb = Color.rgb * RcpMax;
	RGBT.a = Max * rcp(1.0 + Max);
	return RGBT;
}

MaterialFloat3 RGBTDecode(MaterialFloat4 RGBT)
{
	RGBT.a = RGBT.a * rcp(1.0 - RGBT.a);
	return RGBT.rgb * RGBT.a;
}



MaterialFloat4 RGBMEncode( MaterialFloat3 Color )
{
	Color *= 1.0 / 64.0;
	
	float4 rgbm;
	rgbm.a = saturate( max( max( Color.r, Color.g ), max( Color.b, 1e-6 ) ) );
	rgbm.a = ceil( rgbm.a * 255.0 ) / 255.0;
	rgbm.rgb = Color / rgbm.a;
	return rgbm;
}

MaterialFloat4 RGBMEncodeFast( MaterialFloat3 Color )
{
	// 0/0 result written to fixed point buffer goes to zero
	MaterialFloat4 rgbm;
	rgbm.a = dot( Color, 255.0 / 64.0 );
	rgbm.a = ceil( rgbm.a );
	rgbm.rgb = Color / rgbm.a;
	rgbm *= MaterialFloat4( 255.0 / 64.0, 255.0 / 64.0, 255.0 / 64.0, 1.0 / 255.0 );
	return rgbm;
}

MaterialFloat3 RGBMDecode( MaterialFloat4 rgbm, MaterialFloat MaxValue )
{
	return rgbm.rgb * (rgbm.a * MaxValue);
}

MaterialFloat3 RGBMDecode( MaterialFloat4 rgbm )
{
	return rgbm.rgb * (rgbm.a * 64.0f);
}

MaterialFloat4 RGBTEncode8BPC(MaterialFloat3 Color, MaterialFloat Range)
{
	MaterialFloat Max = max(max(Color.r, Color.g), max(Color.b, 1e-6));
	Max = min(Max, Range);

	MaterialFloat4 RGBT;
	RGBT.a = (Range + 1) / Range *  Max / (1 + Max);

	// quantise alpha to 8 bit.
	RGBT.a = ceil(RGBT.a*255.0) / 255.0;
	Max = RGBT.a / (1 + 1 / Range - RGBT.a);

	MaterialFloat RcpMax = rcp(Max);
	RGBT.rgb = Color.rgb * RcpMax;
	return RGBT;
}

MaterialFloat3 RGBTDecode8BPC(MaterialFloat4 RGBT, MaterialFloat Range)
{
	RGBT.a = RGBT.a / (1 + 1 / Range - RGBT.a);
	return RGBT.rgb * RGBT.a;
}

/** Get render target write mask value
  * This gets a bit from a write mask texture created with FRTWriteMaskDecodeCS. Only supprted on some platforms.
  */
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
uint DecodeRTWriteMask(uint2 PixelPos, Texture2D<uint> RTWriteMaskTexture, uint NumEncodedTextures)
{
	uint2 TileIndex = PixelPos >> 3;
	uint2 TileOffset = (PixelPos >> 2) & 1;

	uint Shift = (TileOffset.y * 2 + TileOffset.x) * NumEncodedTextures;
	uint Mask = ~((~0u) << NumEncodedTextures);

	return (RTWriteMaskTexture.Load(uint3(TileIndex, 0)) >> Shift) & Mask;
}
#endif

/** Calculates the ScreenUV given the screen position and an offset fraction. */
float2 CalcScreenUVFromOffsetFraction(float4 ScreenPosition, float2 OffsetFraction)
{
	float2 NDC = ScreenPosition.xy / ScreenPosition.w;
	// Apply the offset in NDC space so that it is consistent regardless of scene color buffer size
	// Clamp to valid area of the screen to avoid reading garbage
	//@todo - soft clamp
	float2 OffsetNDC = clamp(NDC + OffsetFraction * float2(2, -2), -.999f, .999f);
	return float2(OffsetNDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz);
}

float4 GetPerPixelLightAttenuation(float2 UV)
{
	return DecodeLightAttenuation(Texture2DSampleLevel(LightAttenuationTexture, LightAttenuationTextureSampler, UV, 0));
}

// Returns whether or not the given projection matrix is orthographic
bool IsOrthoProjection(float4x4 ViewToClip)
{
	return ViewToClip._44 >= 1.0f;
}

// Return whether View has ortho or perspective projection
bool IsOrthoProjection(ViewState InView)
{
	return IsOrthoProjection(InView.ViewToClip);
}

// Returns whether or not the default view is orthographic
bool IsOrthoProjection()
{
	return IsOrthoProjection(View.ViewToClip);
}

// Returns whether or not the given projection matrix is orthographic via float
float IsOrthoProjectionFloat()
{
	return select(IsOrthoProjection(), 1.0f, 0.0f);
}

// inverse operation of ConvertFromDeviceZ()
// @param SceneDepth (linear in world units, W)
// @return DeviceZ (Z/W)
// WARNING: InvDeviceZToWorldZTransform has a tiny fudge factor added to the projection matrix in order to prevent divide by zero and will cause mismatches with real DeviceZ very far away from camera
float ConvertToDeviceZ(float SceneDepth)
{
	FLATTEN
	if (IsOrthoProjection())
	{
		// Ortho
		return SceneDepth * View.ViewToClip[2][2] + View.ViewToClip[3][2];
	}
	else
	{
		// Perspective
		return 1.0f / ((SceneDepth + View.InvDeviceZToWorldZTransform[3]) * View.InvDeviceZToWorldZTransform[2]);
	}
}

// also see ConvertToDeviceZ()
// @param DeviceZ value that is stored in the depth buffer (Z/W)
// @return SceneDepth (linear in world units, W)
// WARNING: InvDeviceZToWorldZTransform has a tiny fudge factor added to the projection matrix in order to prevent divide by zero and will cause mismatches with real SceneDepth very far away from camera
float ConvertFromDeviceZ(float DeviceZ)
{
	// Supports ortho and perspective, see CreateInvDeviceZToWorldZTransform()
	return DeviceZ * View.InvDeviceZToWorldZTransform[0] + View.InvDeviceZToWorldZTransform[1] + 1.0f / (DeviceZ * View.InvDeviceZToWorldZTransform[2] - View.InvDeviceZToWorldZTransform[3]);
}

float2 AdjustClipToPrevClipForProjectionType(float2 Velocity, float DeviceZ)
{
	if(IsOrthoProjection())
	{
		/*
		* Motion vectors on far distance planes are greatly influenced by slight rotations in camera angle
		* And with Orthographic cameras, this magnifies the velocities over large distances.
		* As a result, we need to counter this by reducing the speed with the DeviceZ
		*
		* Because this is ClipToPrevClip rather than velocity, we use the non - linear device z which works better than linear depth,
		* however if adjusting velocity directly, linear depth should be used instead. 
		*/
		Velocity *= DeviceZ;
	}
	return Velocity;
}

float GetScreenPositionDepth(float4 ScreenPosition)
{
#if RAYTRACINGSHADER
	// Workaround for UE-212146: Accessing global ResolvedView in this case triggers internal compiler error on some platforms.
	return select(IsOrthoProjection(ResolveView()), ConvertFromDeviceZ(ScreenPosition.z), ScreenPosition.w);
#else
	return select(IsOrthoProjection(ResolvedView), ConvertFromDeviceZ(ScreenPosition.z), ScreenPosition.w);
#endif
}

float3 GetCameraVector(ViewState InView, float3 Position, float3 CameraPosition)
{
	return select(IsOrthoProjection(InView), InView.ViewForward, normalize(Position - CameraPosition));
}

float3 GetCameraVector(float3 Position, float3 CameraPosition)
{
	return GetCameraVector(PrimaryView, Position, CameraPosition);
}

float3 GetCameraVectorFromWorldPosition(float3 WorldPosition)
{
	return GetCameraVector(WorldPosition, DFHackToFloat(PrimaryView.WorldCameraOrigin));
}

float3 GetCameraVectorFromTranslatedWorldPosition(ViewState InView, float3 TranslatedWorldPosition)
{
	return GetCameraVector(TranslatedWorldPosition, InView.TranslatedWorldCameraOrigin);
}

float3 GetCameraVectorFromTranslatedWorldPosition(float3 TranslatedWorldPosition)
{
	return GetCameraVectorFromTranslatedWorldPosition(PrimaryView, TranslatedWorldPosition);
}

float GetDistanceToCameraFromViewVector(float3 DistanceVector)
{
	float DistanceToCamera = length(DistanceVector);
	if (IsOrthoProjection())
	{
		/**
		* Position to camera length is not the correct value here, ortho projection needs the distance
		* as the view forward "plane" to the position instead. This length is recalculated here.
		*/
		DistanceToCamera *= DistanceToCamera / dot(DistanceVector, View.ViewForward);
	}
	return DistanceToCamera;
}

float GetDistanceToCameraFromViewVectorSqr(float3 DistanceVector)
{
	if(IsOrthoProjection())
	{
		float Distance = GetDistanceToCameraFromViewVector(DistanceVector);
		return Distance * Distance;
	}
	return dot(DistanceVector, DistanceVector);
}

float2 ScreenPositionToBufferUV(float4 ScreenPosition)
{
	return float2(ScreenPosition.xy / ScreenPosition.w * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz);
}

float3 ScreenVectorFromScreenRect(float4 Position)
{
	//Screen rect does not include scene depth. Perspective projection requires ScreenToTranslated world matrix
	//Ortho Screen does not need the calculation as the screenvector is always the camera direction, so uses ViewForward

	return select(IsOrthoProjection(), View.ViewForward, mul(Position,View.ScreenToTranslatedWorld).xyz);
}

float3 ScreenVectorFromScreenRect(float4 Position, ViewState InView)
{
	//Screen rect does not include scene depth. Perspective projection requires ScreenToTranslated world matrix
	//Ortho Screen does not need the calculation as the screenvector is always the camera direction, so uses ViewForward

	return select(IsOrthoProjection(), InView.ViewForward, mul(Position, InView.ScreenToTranslatedWorld).xyz);
}

// Returns the screen position for projection matrix calculations depending on the type of projection the view is using
float2 GetScreenPositionForProjectionType(float2 ScreenPosition, float SceneDepth)
{
	// For perspective projection matrix, the scene depth is required as part of the clip to view calculations
	// For orthogonal projection matrix, scene depth should not be used
	return select(IsOrthoProjection(), ScreenPosition, ScreenPosition * SceneDepth);
}

float ConvertGivenDepthRadiusForProjectionType(float Radius, float SceneDepth, bool bForceOrthoView = false)
{
	return select(IsOrthoProjection() || bForceOrthoView, Radius, Radius * SceneDepth);
}

float GetDepthPixelRadiusForProjectionType(float SceneDepth)
{
	return View.WorldDepthToPixelWorldRadius.x * SceneDepth + View.WorldDepthToPixelWorldRadius.y;
}

// In many locations we calculate screen ray length using scene depth, this does not work for Ortho
// This code moves that calculation to the view buffer and reduces the SceneDepth option to 1 MAD
float2 GetScreenRayLengthMultiplierForProjectionType(float SceneDepth)
{
	return View.ScreenRayLengthMultiplier.xy * SceneDepth + View.ScreenRayLengthMultiplier.zw;
}

float2 SvPositionToBufferUV(float4 SvPosition)
{
	return SvPosition.xy * View.BufferSizeAndInvSize.zw;
}

// Used for post process shaders which don't need to resolve the view	
float3 SvPositionToTranslatedWorld(float4 SvPosition)
{
	float4 HomWorldPos = mul(float4(SvPosition.xyz, 1), View.SVPositionToTranslatedWorld);

	return HomWorldPos.xyz / HomWorldPos.w;
}

// Used for vertex factory shaders which need to use the resolved view
float3 SvPositionToResolvedTranslatedWorld(float4 SvPosition)
{
	float4 HomWorldPos = mul(float4(SvPosition.xyz, 1), ResolvedView.SVPositionToTranslatedWorld);

	return HomWorldPos.xyz / HomWorldPos.w;
}

// prefer to use SvPositionToTranslatedWorld() for better quality
FDFVector3 SvPositionToWorld(float4 SvPosition)
{
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);
	return DFFastSubtract(TranslatedWorldPosition, PrimaryView.PreViewTranslation);
}

// investigate: doesn't work for usage with PrimaryView.ScreenToWorld, see SvPositionToScreenPosition2()
float4 SvPositionToScreenPosition(float4 SvPosition)
{
	// todo: is already in .w or needs to be reconstructed like this:
//	SvPosition.w = ConvertFromDeviceZ(SvPosition.z);

	float2 PixelPos = SvPosition.xy - View.ViewRectMin.xy;	

	// NDC (NormalizedDeviceCoordinates, after the perspective divide)
	float3 NDCPos = float3( (PixelPos * View.ViewSizeAndInvSize.zw - 0.5f) * float2(2, -2), SvPosition.z);

	// SvPosition.w: so .w has the SceneDepth, some mobile code and the DepthFade material expression wants that
	return float4(NDCPos.xyz, 1) * SvPosition.w;
}

// Used for vertex factory shaders which need to use the resolved view
float4 SvPositionToResolvedScreenPosition(float4 SvPosition)
{
	float2 PixelPos = SvPosition.xy - ResolvedView.ViewRectMin.xy;	

	// NDC (NormalizedDeviceCoordinates, after the perspective divide)
	float3 NDCPos = float3( (PixelPos * ResolvedView.ViewSizeAndInvSize.zw - 0.5f) * float2(2, -2), SvPosition.z);

	// SvPosition.w: so .w has the SceneDepth, some mobile code and the DepthFade material expression wants that
	return float4(NDCPos.xyz, 1) * SvPosition.w;
}

void SvPositionToResolvedScreenPositionDeriv(float4 SvPosition, float2 PPZ_DDX_DDY, float2 W_DDX_DDY, inout float4 ScreenPosition, inout float4 ScreenPositionDDX, inout float4 ScreenPositionDDY)
{
	float2 PixelPos = SvPosition.xy - ResolvedView.ViewRectMin.xy;

	// NDC (NormalizedDeviceCoordinates, after the perspective divide)
	float4 NDCPos = float4((PixelPos * ResolvedView.ViewSizeAndInvSize.zw - 0.5f) * float2(2, -2), SvPosition.z, 1.0f);
	float4 NDCPosDDX = float4(ResolvedView.ViewSizeAndInvSize.z * 2.0f, 0.0f, PPZ_DDX_DDY.x, 0.0f);
	float4 NDCPosDDY = float4(ResolvedView.ViewSizeAndInvSize.w * 2.0f, 0.0f, PPZ_DDX_DDY.y, 0.0f);

	ScreenPosition = NDCPos * SvPosition.w;
	ScreenPositionDDX = NDCPos * W_DDX_DDY.x + NDCPosDDX * SvPosition.w;
	ScreenPositionDDY = NDCPos * W_DDX_DDY.y + NDCPosDDY * SvPosition.w;
}

float3 GetTranslatedWorldCameraPosFromView(ViewState InView, float2 SvPosition, bool bForceAddOrthoHeight = false)
{
	// Get the camera position per pixel in ortho, or just the camera location in perspective.
	if (IsOrthoProjection(InView))
	{
		//When using sky height, for example, the horizon calculation requires bForceAddOrthoHeight to be true to 
		//avoid negative calculations when comparing the relative sky heights + view camera height
		float3 CameraPosition = SvPositionToTranslatedWorld(float4(SvPosition, 1.0f, 1.0f));
		CameraPosition.z += select(bForceAddOrthoHeight, View.ClipToView[1][1], 0.0f);
		return CameraPosition;
	}
	else
	{
		return InView.TranslatedWorldCameraOrigin;
	}
}

FDFVector3 GetWorldCameraPosFromView(ViewState InView, float2 SvPosition, bool bForceAddOrthoHeight = false)
{
	if(IsOrthoProjection(InView))
	{
		//When using sky height, for example, the horizon calculation requires bForceAddOrthoHeight to be true to 
		//avoid negative calculations when comparing the relative sky heights + view camera height
		FDFVector3 CameraPosition = SvPositionToWorld(float4(SvPosition, 1.0f, 1.0f));
		CameraPosition = DFFastAdd(select(bForceAddOrthoHeight, float3(0.0f, 0.0f, View.ClipToView[1][1]), float3(0.0f,0.0f,0.0f)), CameraPosition);
		return CameraPosition;
	}
	else
	{
		return InView.WorldCameraOrigin;
	}
}

float3 GetTranslatedWorldCameraPosFromView(float2 SvPosition, bool bForceAddOrthoHeight = false)
{
	return GetTranslatedWorldCameraPosFromView(PrimaryView, SvPosition, bForceAddOrthoHeight);
}

float3 GetScreenWorldDir(in float4 SVPos)
{
	float2 ScreenPosition = SvPositionToScreenPosition(SVPos).xy;
	const float Depth = 1000000.0f;
	float4 TranslatedWorldPos = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, Depth), Depth, 1), PrimaryView.ScreenToTranslatedWorld);
	return GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPos.xyz);
}

float2 SvPositionToViewportUV(float4 SvPosition)
{
	// can be optimized from 2SUB+2MUL to 2MAD
	float2 PixelPos = SvPosition.xy - ResolvedView.ViewRectMin.xy;	

	return PixelPos.xy * ResolvedView.ViewSizeAndInvSize.zw;
}

float2 BufferUVToViewportUV(float2 BufferUV)
{
#if INSTANCED_STEREO || MOBILE_MULTI_VIEW
	float2 PixelPos = BufferUV.xy * ResolvedView.BufferSizeAndInvSize.xy - ResolvedView.ViewRectMin.xy;
	return PixelPos.xy * ResolvedView.ViewSizeAndInvSize.zw;
#else
	float2 PixelPos = BufferUV.xy * View.BufferSizeAndInvSize.xy - View.ViewRectMin.xy;
	return PixelPos.xy * View.ViewSizeAndInvSize.zw;
#endif
}

float2 ViewportUVToBufferUV(float2 ViewportUV)
{
#if INSTANCED_STEREO || MOBILE_MULTI_VIEW
	float2 PixelPos = ViewportUV * ResolvedView.ViewSizeAndInvSize.xy;
	return (PixelPos + ResolvedView.ViewRectMin.xy) * ResolvedView.BufferSizeAndInvSize.zw;
#else
	float2 PixelPos = ViewportUV * View.ViewSizeAndInvSize.xy;
	return (PixelPos + View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
#endif
}

// Maps standard viewport UV to screen position.
float2 ViewportUVToScreenPos(float2 ViewportUV)
{
	return float2(2 * ViewportUV.x - 1, 1 - 2 * ViewportUV.y);
}

float2 ScreenPosToViewportUV(float2 ScreenPos)
{
	return float2(0.5 + 0.5 * ScreenPos.x, 0.5 - 0.5 * ScreenPos.y);
}

// Maps standard viewport UV to an unprojected viewpos.
// Viewpos can then be achieved via out.xy / out.z
float3 ScreenToViewPos(float2 ViewportUV, float SceneDepth)
{
	float2 ProjViewPos;

	ProjViewPos.x =  ViewportUV.x * View.ScreenToViewSpace.x  + View.ScreenToViewSpace.z;
	ProjViewPos.y =  ViewportUV.y * View.ScreenToViewSpace.y  + View.ScreenToViewSpace.w;
	return float3(GetScreenPositionForProjectionType(ProjViewPos, SceneDepth), SceneDepth);
}

// ----------------------------

/** 
 * aligns the clip space position so that it can be used as a texture coordinate
 * to properly align in screen space
 */
MaterialFloat2 ScreenAlignedPosition( float4 ScreenPosition )
{
	return MaterialFloat2(ScreenPositionToBufferUV(ScreenPosition));
}

/** 
 * Aligns the [0,1] UV to match the view within the backbuffer
 */
MaterialFloat2 ScreenAlignedUV( MaterialFloat2 UV )
{
	return (UV*MaterialFloat2(2,-2) + MaterialFloat2(-1,1))*View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
}

/**
 * Compute viewport coordinates from the given fragment coordinates.
 */
MaterialFloat2 GetViewportCoordinates(MaterialFloat2 InFragmentCoordinates)
{
	return InFragmentCoordinates;
}

/**
 * Unpack a normal stored in a normal map. The X and Y components are rescaled from [0,1] to [-1,1] and Z is reconstructed.
 */
MaterialFloat4 UnpackNormalMap( MaterialFloat4 TextureSample )
{
	#if DXT5_NORMALMAPS
		MaterialFloat2 NormalXY = TextureSample.ag;
	#elif LA_NORMALMAPS
		MaterialFloat2 NormalXY = TextureSample.ra;
	#else
		MaterialFloat2 NormalXY = TextureSample.rg;
	#endif

	NormalXY = NormalXY * MaterialFloat2(2.0f,2.0f) - MaterialFloat2(1.0f,1.0f);
	MaterialFloat NormalZ = sqrt( saturate( 1.0f - dot( NormalXY, NormalXY ) ) );
	return MaterialFloat4( NormalXY.xy, NormalZ, 1.0f );
}

MaterialFloat4 UnpackNormalMapVT(MaterialFloat4 TextureSample)
{
	// UnpackNormalMap expects a specific channel layout, which may differ between regular and virtual textures on certain platforms (e.g., ASTC LA)
#if LA_NORMALMAPS
	return UnpackNormalMap(TextureSample.rggg);
#else
	return UnpackNormalMap(TextureSample);
#endif
}

// Antialiased version of a binary comparison between ThresholdConst and a texture channel.
float AntialiasedTextureMask( Texture2D Tex, SamplerState Sampler, float2 UV, float ThresholdConst, int Channel )
{
	// By setting MaskConst to 0001, 0010, 0100 or 1000 individual channels can be chosen (the compiler should be able to optimize that).
	MaterialFloat4 MaskConst = MaterialFloat4(Channel == 0, Channel == 1, Channel == 2, Channel == 3);

	// border width in pixels, for antialiasing 1 .. 1.5 is good but 1.0 is good for optimizations 
	const float WidthConst = 1.0f;			
	float InvWidthConst = 1 / WidthConst;

	// Problem:

	// A simple texture lookup with a comparison against some thresold value allows to get a mask useful
	// for many purposes (e.g. text rendering, signs, oil/water/paint). Antialiased masks look much better
	// and mip mapping provides that but only for minification. So when the texture resolution is lower than
	// the rendering size results get blurry.

	// Idea:

	// We compute the distance to the threshold line in pixels (with subpixel precision). We can visualize
	// the problem as a heightmap that intersects a axis aligned plane at the threshold height. Only surface
	// above the threshold plane contributes to the mask. Looking at one pixel the heightmap can be approximated
	// by a plane. We can easily get the plane center value form a texture lookup and get the plane equation from
	// ddx and ddy of that value (only one value per 2x2 block) or some other more precise method. We can reduce the
	// 3d problem to 2d (looking at the steepest angle only) and the resulting value tells us how much the texture value
	// changes for one pixel. This allows us to scale and bias (threshold) the texture value the so it maps to the
	// distance function. We rescaling the distance to 0.5 coverage at the line, >1 MaterialFloat a pixel inside and <0 MaterialFloat
	// a pixel outside. Clamping this value in the range from 0..1 gives us a good approximation of the pixel coverage.

	// We tried multiple possible implementations - this is the cheapest and looks ok is most cases.
	// If quality improvements are needed we can add an option to the node later on.
	float Result;
	{
		// optimized, ddx/ddy only for every 2x2 block (bad for distant stuff)
		float Sample1 = dot(MaskConst, Texture2DSample(Tex, Sampler, UV));

		// compute the derivatives of the texture content
		float2 TexDD = float2(DDX(Sample1), DDY(Sample1));

		float TexDDLength = max(abs(TexDD.x), abs(TexDD.y)); 
		float Top = InvWidthConst * (Sample1 - ThresholdConst);
		Result = Top / TexDDLength + ThresholdConst;
	}

	Result = saturate(Result);	// no always needed (e.g. DX9 framebuffer blending)

	return Result;
}

// While RepeatSize is a float here, the expectation is that it would be largely integer values coming in from the UI. The downstream logic assumes
// floats for all called functions (NoiseTileWrap) and this prevents any float-to-int conversion errors from automatic type conversion.
float Noise3D_Multiplexer(int Function, float3 Position, int Quality, bool bTiling, float RepeatSize)
{
	// verified, HLSL compiled out the switch if Function is a constant
	switch(Function)
	{
		case 0:
			return SimplexNoise3D_TEX(Position);
		case 1:
			return GradientNoise3D_TEX(Position, bTiling, RepeatSize);
		case 2:
			return FastGradientPerlinNoise3D_TEX(Position);
		case 3:
			return GradientNoise3D_ALU(Position, bTiling, RepeatSize);
		case 4:
			return ValueNoise3D_ALU(Position, bTiling, RepeatSize);
		default:
			return VoronoiNoise3D_ALU(Position, Quality, bTiling, RepeatSize, true).w * 2. - 1.;
	}
	return 0;
}

// @param LevelScale usually 2 but higher values allow efficient use of few levels
// @return in user defined range (OutputMin..OutputMax)
MaterialFloat MaterialExpressionNoise(float3 Position, float Scale, int Quality, int Function, bool bTurbulence, uint Levels, float OutputMin, float OutputMax, float LevelScale, float FilterWidth, bool bTiling, float RepeatSize)
{
	Position *= Scale;
	FilterWidth *= Scale;

	float Out = 0.0f;
	float OutScale = 1.0f;
	float InvLevelScale = 1.0f / LevelScale;
	
	LOOP for(uint i = 0; i < Levels; ++i)
	{
		// fade out noise level that are too high frequent (not done through dynamic branching as it usually requires gradient instructions)
		OutScale *= saturate(1.0 - FilterWidth);

		if(bTurbulence)
		{
			Out += abs(Noise3D_Multiplexer(Function, Position, Quality, bTiling, RepeatSize)) * OutScale;
		}
		else
		{
			Out += Noise3D_Multiplexer(Function, Position, Quality, bTiling, RepeatSize) * OutScale;
		}

		Position *= LevelScale;
		RepeatSize *= LevelScale;
		OutScale *= InvLevelScale;
		FilterWidth *= LevelScale;
	}

	if(!bTurbulence)
	{
		// bring -1..1 to 0..1 range
		Out = Out * 0.5f + 0.5f;
	}

	// Out is in 0..1 range
	return lerp(OutputMin, OutputMax, Out);
}

MaterialFloat MaterialExpressionNoise(FDFVector3 LWCPosition, float Scale, int Quality, int Function, bool bTurbulence, uint Levels, float OutputMin, float OutputMax, float LevelScale, float FilterWidth, bool bTiling, float RepeatSize)
{
	const float TileSize = 65536;
	float3 Position = abs(TileSize * DFFracDemote(DFDivideByPow2(LWCPosition, TileSize))); // Mirror and repeat every TileSize
	return MaterialExpressionNoise(Position, Scale, Quality, Function, bTurbulence, Levels, OutputMin, OutputMax, LevelScale, FilterWidth, bTiling, RepeatSize);
}

MaterialFloat MaterialExpressionNoise(FLWCVector3 LWCPosition, float Scale, int Quality, int Function, bool bTurbulence, uint Levels, float OutputMin, float OutputMax, float LevelScale, float FilterWidth, bool bTiling, float RepeatSize)
{
	float3 Position = LWCNormalizeTile(LWCPosition).Offset;
	return MaterialExpressionNoise(Position, Scale, Quality, Function, bTurbulence, Levels, OutputMin, OutputMax, LevelScale, FilterWidth, bTiling, RepeatSize);
}

// Material node for noise functions returning a vector value
// @param LevelScale usually 2 but higher values allow efficient use of few levels
// @return in user defined range (OutputMin..OutputMax)
MaterialFloat4 MaterialExpressionVectorNoise(MaterialFloat3 Position, int Quality, int Function, bool bTiling, float TileSize)
{
	float4 result = float4(0,0,0,1);
	float3x4 Jacobian = JacobianSimplex_ALU(Position, bTiling, TileSize);	// compiled out if not used

	// verified, HLSL compiled out the switch if Function is a constant
	switch (Function)
	{
	case 0:	// Cellnoise
		result.xyz = float3(Rand3DPCG16(int3(floor(NoiseTileWrap(Position, bTiling, TileSize))))) / 0xffff;
		break;
	case 1: // Color noise
		result.xyz = float3(Jacobian[0].w, Jacobian[1].w, Jacobian[2].w);
		break;
	case 2: // Gradient
		result = Jacobian[0];
		break;
	case 3: // Curl
		result.xyz = float3(Jacobian[2][1] - Jacobian[1][2], Jacobian[0][2] - Jacobian[2][0], Jacobian[1][0] - Jacobian[0][1]);
		break;
	default: // Voronoi
		result = VoronoiNoise3D_ALU(Position, Quality, bTiling, TileSize, false);
		break;
	}
	return result;
}


/*
* Clips a ray to an AABB.  Does not handle rays parallel to any of the planes.
*
* @param RayOrigin - The origin of the ray in world space.
* @param RayEnd - The end of the ray in world space.  
* @param BoxMin - The minimum extrema of the box.
* @param BoxMax - The maximum extrema of the box.
* @return - Returns the closest intersection along the ray in x, and furthest in y.  
*			If the ray did not intersect the box, then the furthest intersection <= the closest intersection.
*			The intersections will always be in the range [0,1], which corresponds to [RayOrigin, RayEnd] in worldspace.
*			To find the world space position of either intersection, simply plug it back into the ray equation:
*			WorldPos = RayOrigin + (RayEnd - RayOrigin) * Intersection;
*/
float2 LineBoxIntersect(float3 RayOrigin, float3 RayEnd, float3 BoxMin, float3 BoxMax)
{
	float3 InvRayDir = 1.0f / (RayEnd - RayOrigin);
	
	//find the ray intersection with each of the 3 planes defined by the minimum extrema.
	float3 FirstPlaneIntersections = (BoxMin - RayOrigin) * InvRayDir;
	//find the ray intersection with each of the 3 planes defined by the maximum extrema.
	float3 SecondPlaneIntersections = (BoxMax - RayOrigin) * InvRayDir;
	//get the closest of these intersections along the ray
	float3 ClosestPlaneIntersections = min(FirstPlaneIntersections, SecondPlaneIntersections);
	//get the furthest of these intersections along the ray
	float3 FurthestPlaneIntersections = max(FirstPlaneIntersections, SecondPlaneIntersections);

	float2 BoxIntersections;
	//find the furthest near intersection
	BoxIntersections.x = max(ClosestPlaneIntersections.x, max(ClosestPlaneIntersections.y, ClosestPlaneIntersections.z));
	//find the closest far intersection
	BoxIntersections.y = min(FurthestPlaneIntersections.x, min(FurthestPlaneIntersections.y, FurthestPlaneIntersections.z));
	//clamp the intersections to be between RayOrigin and RayEnd on the ray
	return saturate(BoxIntersections);
}

/** Computes distance from an AABB to a point in space. */
MaterialFloat ComputeDistanceFromBoxToPoint(MaterialFloat3 Mins, MaterialFloat3 Maxs, MaterialFloat3 InPoint)
{
	MaterialFloat3 DistancesToMin = select(InPoint < Mins, abs(InPoint - Mins), MaterialFloat(0.0));
	MaterialFloat3 DistancesToMax = select(InPoint > Maxs, abs(InPoint - Maxs), MaterialFloat(0.0));

	//@todo - this is actually incorrect, it gives manhattan distance
	MaterialFloat Distance = dot(DistancesToMin, 1);
	Distance += dot(DistancesToMax, 1);
	return Distance;
}

/** Computes squared distance from a point in space to an AABB. */
MaterialFloat ComputeSquaredDistanceFromBoxToPoint(MaterialFloat3 BoxCenter, MaterialFloat3 BoxExtent, MaterialFloat3 InPoint)
{
	MaterialFloat3 AxisDistances = max(abs(InPoint - BoxCenter) - BoxExtent, 0);
	return dot(AxisDistances, AxisDistances);
}

/** Computes distance from point inside an AABB to the AABB's surface. */
float ComputeDistanceFromBoxToPointInside(float3 BoxCenter, float3 BoxExtent, float3 InPoint)
{
	float3 DistancesToMin = max(InPoint - BoxCenter + BoxExtent, 0);
	float3 DistancesToMax = max(BoxCenter + BoxExtent - InPoint, 0);
	float3 ClosestDistances = min(DistancesToMin, DistancesToMax);
	return min(ClosestDistances.x, min(ClosestDistances.y, ClosestDistances.z));
}

bool RayHitSphere(float3 RayOrigin, float3 UnitRayDirection, float3 SphereCenter, float SphereRadius)
{
	float3 ClosestPointOnRay = max(0, dot(SphereCenter - RayOrigin, UnitRayDirection)) * UnitRayDirection;
	float3 CenterToRay = RayOrigin + ClosestPointOnRay - SphereCenter;
	return dot(CenterToRay, CenterToRay) <= Square(SphereRadius);
}

bool RaySegmentHitSphere(float3 RayOrigin, float3 UnitRayDirection, float RayLength, float3 SphereCenter, float SphereRadius)
{
	float DistanceAlongRay = dot(SphereCenter - RayOrigin, UnitRayDirection);
	float3 ClosestPointOnRay = DistanceAlongRay * UnitRayDirection;
	float3 CenterToRay = RayOrigin + ClosestPointOnRay - SphereCenter;
	return dot(CenterToRay, CenterToRay) <= Square(SphereRadius) && DistanceAlongRay > -SphereRadius && DistanceAlongRay - SphereRadius < RayLength;
}

/**
 * Returns near intersection in x, far intersection in y, or both -1 if no intersection.
 * RayDirection does not need to be unit length.
 */
float2 RayIntersectSphere(float3 RayOrigin, float3 RayDirection, float4 Sphere)
{
	float3 LocalPosition = RayOrigin - Sphere.xyz;
	float LocalPositionSqr = dot(LocalPosition, LocalPosition);

	float3 QuadraticCoef;
	QuadraticCoef.x = dot(RayDirection, RayDirection);
	QuadraticCoef.y = 2 * dot(RayDirection, LocalPosition);
	QuadraticCoef.z = LocalPositionSqr - Sphere.w * Sphere.w;

	float Discriminant = QuadraticCoef.y * QuadraticCoef.y - 4 * QuadraticCoef.x * QuadraticCoef.z;

	float2 Intersections = -1;

	// Only continue if the ray intersects the sphere
	FLATTEN
	if (Discriminant >= 0)
	{
		float SqrtDiscriminant = sqrt(Discriminant);
		Intersections = (-QuadraticCoef.y + float2(-1, 1) * SqrtDiscriminant) / (2 * QuadraticCoef.x);
	}

	return Intersections;
}

/** Transforms a vector from tangent space to world space */
MaterialFloat3 TransformTangentVectorToWorld(MaterialFloat3x3 TangentToWorld, MaterialFloat3 InTangentVector)
{
	// Transform directly to world space
	// The vector transform is optimized for this case, only one vector-matrix multiply is needed
	return mul(InTangentVector, TangentToWorld);
}

/** Transforms a vector from world space to tangent space */
MaterialFloat3 TransformWorldVectorToTangent(MaterialFloat3x3 TangentToWorld, MaterialFloat3 InWorldVector)
{
	// Transform from world to tangent space with the transpose of TangentToWorld (achieved by swapping vector / matrix multiply order)
	// Note that the transpose is only equal to the inverse for orthonormal matrices - aka only uniform scaling
	return mul(TangentToWorld, InWorldVector);
}

float3 TransformWorldVectorToView(float3 InTangentVector)
{
	// Transform from world to view space
	return mul(InTangentVector, (float3x3)ResolvedView.TranslatedWorldToView);
}

/** Computes the distance from the center to the edge of an AABB with the given extents in the given direction. */
MaterialFloat GetBoxPushout(MaterialFloat3 Normal,MaterialFloat3 Extent)
{
	return dot(abs(Normal * Extent), MaterialFloat3(1.0f, 1.0f, 1.0f));
}

/** Generates arbitrary but valid perpendicular unit vectors to ZAxis.  ZAxis should be unit length. */
void GenerateCoordinateSystem(float3 ZAxis, out float3 XAxis, out float3 YAxis)
{
	if (abs(ZAxis.x) > abs(ZAxis.y))
	{
		float InverseLength = 1.0f / sqrt(dot(ZAxis.xz, ZAxis.xz));
		XAxis = float3(-ZAxis.z * InverseLength, 0.0f, ZAxis.x * InverseLength);
	}
	else
	{
		float InverseLength = 1.0f / sqrt(dot(ZAxis.yz, ZAxis.yz));
		XAxis = float3(0.0f, ZAxis.z * InverseLength, -ZAxis.y * InverseLength);
	}

	YAxis = cross(ZAxis, XAxis);
}

// Define passthrough implementations of EvaluateAttributeAtSample for non-D3D11 platforms.
#if !(SM6_PROFILE || SM5_PROFILE)
	float EvaluateAttributeAtSample(float Attribute,uint SampleIndex) { return Attribute; }
	float2 EvaluateAttributeAtSample(float2 Attribute,uint SampleIndex) { return Attribute; }
	float3 EvaluateAttributeAtSample(float3 Attribute,uint SampleIndex) { return Attribute; }
	float4 EvaluateAttributeAtSample(float4 Attribute,uint SampleIndex) { return Attribute; }
#endif

/** Output of the screen vertex shader. */
struct FScreenVertexOutput
{
	noperspective float2 UV : TEXCOORD0;
	float4 Position : SV_POSITION;
};

/** Whether HasPixelAnimation is encoded in the velocity texture. Matches VelocityEncodeHasPixelAnimation() */
#define VELOCITY_ENCODE_HAS_PIXEL_ANIMATION VELOCITY_ENCODE_DEPTH

/** Whether we use more bits to simulate temporal responsiveness **/
#define VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS (VELOCITY_SUPPORT_TEMPORAL_RESPONSIVENESS & VELOCITY_ENCODE_HAS_PIXEL_ANIMATION)

#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
	#define TEMPORAL_RESPONSIVENESS_MASK	0x3
	#define VELOCITY_Z_LOW_MASK			0xFFFC
#else
	#define TEMPORAL_RESPONSIVENESS_MASK	0x1
	#define VELOCITY_Z_LOW_MASK			0xFFFE
#endif
	
// for velocity rendering, motionblur and temporal AA
// velocity needs to support -2..2 screen space range for x and y
// texture is 16bit 0..1 range per channel
ENCODED_VELOCITY_TYPE EncodeVelocityToTexture(float3 V, uint TemporalResponsivenessMask)
{
	#if VELOCITY_ENCODE_GAMMA
		V.xy = sign(V.xy) * sqrt(abs(V.xy)) * (2.0 / sqrt(2.0));
	#endif

	// 0.499f is a value smaller than 0.5f to avoid using the full range to use the clear color (0,0) as special value
	// 0.5f to allow for a range of -2..2 instead of -1..1 for really fast motions for temporal AA
	float4 EncodedV;
	EncodedV.xy = V.xy * (0.499f * 0.5f) + 32767.0f / 65535.0f;

	#if VELOCITY_ENCODE_DEPTH
		uint Vz = asuint(V.z);

		EncodedV.z = saturate(float((Vz >> 16) & 0xFFFF) * rcp(65535.0f) + (0.1 / 65535.0f));
		EncodedV.w = saturate(float(((Vz >>  0) & VELOCITY_Z_LOW_MASK) | (TemporalResponsivenessMask & TEMPORAL_RESPONSIVENESS_MASK)) * rcp(65535.0f) + (0.1 / 65535.0f));
	#else
		EncodedV.zw = 0.0;
	#endif

#if COMPILER_GLSL_ES3_1
	return uint4(EncodedV * 65535.0 + 0.5f);
#else
	return EncodedV;
#endif
}

ENCODED_VELOCITY_TYPE EncodeVelocityToTexture(float3 V,  bool bHasPixelAnimation, float TemporalResponsiveness)
{
	uint TemporalResponsivenessMask = uint(bHasPixelAnimation);
#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
	TemporalResponsivenessMask |= select(TemporalResponsiveness > 0.5f, 3U, uint(TemporalResponsiveness>0));
#endif
	return EncodeVelocityToTexture(V, TemporalResponsivenessMask);
}

ENCODED_VELOCITY_TYPE EncodeVelocityToTexture(float3 V, bool bHasPixelAnimation)
{
	return EncodeVelocityToTexture(V, /* TemporalResponsivenessMask = */uint(bHasPixelAnimation));
}

ENCODED_VELOCITY_TYPE EncodeVelocityToTexture(float3 V)
{
	return EncodeVelocityToTexture(V, /* TemporalResponsivenessMask = */ 0);
}

// see EncodeVelocityToTexture()
float3 DecodeVelocityFromTexture(ENCODED_VELOCITY_TYPE InEncodedV)
{
#if COMPILER_GLSL_ES3_1
	float4 EncodedV = InEncodedV / 65535.0f;
#else
	float4 EncodedV = InEncodedV;
#endif
	const float InvDiv = 1.0f / (0.499f * 0.5f);

	float3 V;
	V.xy = EncodedV.xy * InvDiv - 32767.0f / 65535.0f * InvDiv;

	#if VELOCITY_ENCODE_DEPTH
		V.z = asfloat((uint(round(EncodedV.z * 65535.0f)) << 16) | (uint(round(EncodedV.w * 65535.0f)) & VELOCITY_Z_LOW_MASK));
	#else
		V.z = 0.0;
	#endif

	#if VELOCITY_ENCODE_GAMMA
		V.xy = (V.xy * abs(V.xy)) * 0.5;
	#endif

	return V;
}

/** Returns whether the opaque material drawing velocity had the UMaterial::bHasPixelAnimation, which means the geometric velocity might not accurately represent how the pixel moves. */
#if VELOCITY_ENCODE_HAS_PIXEL_ANIMATION
bool DecodeHasPixelAnimationFromVelocityTexture(ENCODED_VELOCITY_TYPE EncodedV)
{
	return (uint(round(EncodedV.w * 65535.0f)) & 0x1) != 0x0;
}
#endif

#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
bool DecodeDisableAnimationAAFromVelocityTexture(ENCODED_VELOCITY_TYPE EncodedV)
{
	return (uint(round(EncodedV.w * 65535.0f)) & 0x2) != 0x0;
}
#endif

// Used for the Global Illumination in the GIReplace material expression
bool GetGIReplaceState()
{
#if REFLECTIVE_SHADOW_MAP
	return true;
#else
	return false;
#endif
}

// Used for the Nanite path in the NaniteReplace material expression
bool GetNaniteReplaceState()
{
#if IS_NANITE_PASS
	return true;
#else
	return false;
#endif
}

bool GetRayTracingQualitySwitch()
{
#if (RAYTRACINGSHADER || LUMEN_CARD_CAPTURE) && !PATH_TRACING
	return true;
#else
	return false;
#endif
}

#if !PATH_TRACING && !PATH_TRACING_POST_PROCESS_MATERIAL
// Default implementations for non-path traced shaders
// The actual implementations used for the path tracer exist in PathTracingShaderUtils.h to minimize cruft in Common.ush
bool GetPathTracingQualitySwitch() { return false;  }
bool GetPathTracingIsShadow() { return false; }
bool GetPathTracingIsIndirectDiffuse() { return false; }
bool GetPathTracingIsIndirectSpecular() { return false; }
bool GetPathTracingIsIndirectVolume() { return false; }
#endif

bool GetLightmassReplaceState()
{
#if RAYTRACINGSHADER && PATH_TRACING && SIMPLIFIED_MATERIAL_SHADER
	return true;
#else
	return false;
#endif
}

struct FWriteToSliceGeometryOutput
{
	FScreenVertexOutput Vertex;
	uint LayerIndex : SV_RenderTargetArrayIndex;
};


/** Used for calculating vertex positions and UVs when drawing with DrawRectangle */
void DrawRectangle(
	in float4 InPosition,
	in float2 InTexCoord,
	out float4 OutPosition,
	out float2 OutTexCoord)
{
	OutPosition = InPosition;
	OutPosition.xy = -1.0f + 2.0f * (DrawRectangleParameters.PosScaleBias.zw + (InPosition.xy * DrawRectangleParameters.PosScaleBias.xy)) * DrawRectangleParameters.InvTargetSizeAndTextureSize.xy;
	OutPosition.xy *= float2( 1, -1 );
	OutTexCoord.xy = (DrawRectangleParameters.UVScaleBias.zw + (InTexCoord.xy * DrawRectangleParameters.UVScaleBias.xy)) * DrawRectangleParameters.InvTargetSizeAndTextureSize.zw;
}

/** Helper variant for vertex shaders which need a separate output for SV_POSITION and packed UV / Screen position */
void DrawRectangle(
	in float4 InPosition,
	in float2 InTexCoord,
	out float4 OutPosition,
	out float4 OutUVAndScreenPos)
{
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUVAndScreenPos.xy);
	OutUVAndScreenPos.zw = OutPosition.xy;
}

/** Used for calculating vertex positions when drawing with DrawRectangle */
void DrawRectangle(in float4 InPosition, out float4 OutPosition)
{
	OutPosition = InPosition;
	OutPosition.xy = -1.0f + 2.0f * (DrawRectangleParameters.PosScaleBias.zw + (InPosition.xy * DrawRectangleParameters.PosScaleBias.xy)) * DrawRectangleParameters.InvTargetSizeAndTextureSize.xy;
	OutPosition.xy *= float2( 1, -1 );
}

//Since some platforms don't remove Nans in saturate calls, 
//SafeSaturate function will remove nan/inf.    
//Can be expensive, only call when there's a good reason to expect Nans.
//D3D saturate actually turns NaNs -> 0  since it does the max(0.0f, value) first, and D3D NaN rules specify the non-NaN operand wins in such a case.  
//See: https://docs.microsoft.com/en-us/windows/desktop/direct3dhlsl/saturate
#define SafeSaturate_Def(type)\
type SafeSaturate(type In) \
{\
	return saturate(In);\
}

SafeSaturate_Def(float)
SafeSaturate_Def(float2)
SafeSaturate_Def(float3)
SafeSaturate_Def(float4)

// By default HLSL compilers make assumptions about the non-NaN nature of most inputs. Otherwise, most of its optimizations would be invalid, e.g. assuming multiply by 0 equals 0.
// Unfortunately, as a consequence of this, the native infinite()/isnan()/isinf() intrinsics frequently get silently optimized away. To avoid this hazard we manually implement the same logic.
// These are expected to generate identical code as native intrinsics would (when working properly). The PositiveFinite versions are faster than the native.
// Also note that FXC has a quirk in which the pattern b? a : 0 is implemented with an AND instead of the more natural MOVC, i.e. optimizing for D3D bytecode instead of real-world HW instructions.
// So drivers detect this pattern and convert back to a conditional move. HLSLcc instead converts to mix(), so NaNs end up propagating through despite our efforts to kill them (see UE-66179).
// So we break this pattern using !b? 0 : a instead, which generates a MOVC.
#define IsAndMakeFinite_Def(type, booltype)\
booltype IsFinite(type In) \
{\
	return (asuint(In) & 0x7F800000) != 0x7F800000; \
}\
booltype IsPositiveFinite(type In) \
{\
	return asuint(In) < 0x7F800000; \
}\
type MakeFinite(type In) \
{\
    return select(!IsFinite(In), 0.0, In); \
}\
type MakePositiveFinite(type In) \
{\
    return select(!IsPositiveFinite(In), 0.0, In); \
}

IsAndMakeFinite_Def(float,  bool)
IsAndMakeFinite_Def(float2, bool2)
IsAndMakeFinite_Def(float3, bool3)
IsAndMakeFinite_Def(float4, bool4)

// NOTE: The raytraced implementation of the ShadowPassSwitch node is kept in RayTracingShaderUtils.ush as it needs to access per ray information.
#if RAYHITGROUPSHADER == 0
// Experimental way to allow adjusting the OpacityMask for shadow map rendering of masked materials.
// This is exposed via the ShadowPassSwitch material node. This can also be accessed with a Custom
// material node. If this turns out to be very useful we can expose as MaterialFunction
// and potentially expose other queries as well (e.g. SkeletalMesh, HitProxy, ).
// @return 0:no, 1:yes
bool GetShadowReplaceState()
{
#if SHADOW_DEPTH_SHADER
	return true;
#else
	return false;
#endif
}

float IsShadowDepthShader()
{
	return GetShadowReplaceState() ? 1.0f : 0.0f;
}

#endif // RAYHITGROUPSHADER == 0

bool GetReflectionCapturePassSwitchState()
{
	return View.RenderingReflectionCaptureMask > 0.0f;
}


#define TERRAIN_ZSCALE (1.0f/128.0f)

// Decodes a value which was packed into two 8 bit channels
float DecodePackedTwoChannelValue(float2 PackedHeight)
{
	return PackedHeight.x * 255.0 * 256.0 + PackedHeight.y * 255.0;
}

float DecodeHeightValue(float InValue)
{
	return (InValue - 32768.0) * TERRAIN_ZSCALE;
}

float DecodePackedHeight(float2 PackedHeight)
{
	return DecodeHeightValue(DecodePackedTwoChannelValue(PackedHeight));
}

/** Reverses all the <BitCount> lowest significant bits. */
uint ReverseBitsN(uint Bitfield, const uint BitCount)
{
	return reversebits(Bitfield) >> (32 - BitCount);
}

// Remap 2D array index to flattened 1D array index
inline uint Flatten2D(uint2 Coord, uint2 Dim)
{
	return Coord.x + Coord.y * Dim.x;
}

// Remap flattened array index to 2D array index
inline uint2 Unflatten2D(uint Index, uint2 Dim)
{
	return uint2(Index % Dim.x, Index / Dim.x);
}

uint2 ZOrder2D(uint Index, const uint SizeLog2)
{
	uint2 Coord = 0; 

	UNROLL
	for (uint i = 0; i < SizeLog2; i++)
	{
		Coord.x |= ((Index >> (2 * i + 0)) & 0x1) << i;
		Coord.y |= ((Index >> (2 * i + 1)) & 0x1) << i;
	}

	return Coord;
}

uint3 ZOrder3D(uint Index, const uint SizeLog2)
{
    uint3 Coord = 0;
    
    UNROLL
    for (uint i = 0; i < SizeLog2; i++)
    {
        Coord.x |= ((Index >> (3 * i + 0)) & 0x1) << i;
        Coord.y |= ((Index >> (3 * i + 1)) & 0x1) << i;
        Coord.z |= ((Index >> (3 * i + 2)) & 0x1) << i;
    }

    return Coord;
}

uint ZOrder3DEncode(uint3 Coord, const uint SizeLog2)
{
    uint Index = 0;
    
    UNROLL
    for (uint i = 0; i < SizeLog2; i++)
    {
        Index |= ((Coord.x >> i) & 0x1) << (3 * i + 0);
        Index |= ((Coord.y >> i) & 0x1) << (3 * i + 1);
        Index |= ((Coord.z >> i) & 0x1) << (3 * i + 2);
    }

    return Index;
}

uint RoundUpToPowerOfTwo(uint Val)
{
	return Val > 1 ? 1u << (firstbithigh(Val - 1) + 1) : 1;
}

uint DivideAndRoundUp(uint Dividend, uint Divisor)
{
	return (Dividend + Divisor - 1) / Divisor;
}

// Optimised for power of two because it relies on a division done using bit shift
uint DivideAndRoundUp(uint Dividend, uint Divisor, uint DivisorAsBitShift)
{
	return (Dividend + Divisor - 1) >> DivisorAsBitShift;
}

#define DivideAndRoundUp2(Dividend) DivideAndRoundUp((Dividend),    2u,  1u)
#define DivideAndRoundUp4(Dividend) DivideAndRoundUp((Dividend),    4u,  2u)
#define DivideAndRoundUp8(Dividend) DivideAndRoundUp((Dividend),    8u,  3u)
#define DivideAndRoundUp16(Dividend) DivideAndRoundUp((Dividend),  16u,  4u)
#define DivideAndRoundUp32(Dividend) DivideAndRoundUp((Dividend),  32u,  5u)
#define DivideAndRoundUp64(Dividend) DivideAndRoundUp((Dividend),  64u,  6u)
#define DivideAndRoundUp128(Dividend) DivideAndRoundUp((Dividend), 128u, 7u)
#define DivideAndRoundUp256(Dividend) DivideAndRoundUp((Dividend), 256u, 8u)

float UNorm16ToF32(uint UNorm)
{
	return (UNorm & 0xFFFFu) / 65535.0f;
}

float SNorm16ToF32(uint SNorm)
{
	return UNorm16ToF32(SNorm) * 2.0f - 1.0f;
}

float ComputeZSliceFromDepth(float3 GridZParams, float SceneDepth)
{
	return log2(SceneDepth * GridZParams.x + GridZParams.y) * GridZParams.z;
}

float ComputeNormalizedZSliceFromDepth(float3 GridZParams, float InvGridSizeZ, float SceneDepth)
{
	return ComputeZSliceFromDepth(GridZParams, SceneDepth) * InvGridSizeZ;
}

float ComputeDepthFromZSlice(float3 GridZParams, float ZSlice)
{
	float SliceDepth = (exp2(ZSlice / GridZParams.z) - GridZParams.y) / GridZParams.x;
	return SliceDepth;
}

// see PixelShaderOutputCommon
struct FPixelShaderIn
{
	// read only
	float4 SvPosition;

	// Pixel Shader InCoverage, only usable if PIXELSHADEROUTPUT_COVERAGE is 1
	uint Coverage;

	//
	bool bIsFrontFace;
};
// see PixelShaderOutputCommon
struct FPixelShaderOut
{
	// [0..7], only usable if PIXELSHADEROUTPUT_MRT0, PIXELSHADEROUTPUT_MRT1, ... is 1
	float4 MRT[8];
	
	// Explicit uint output specific to Substrate.
	uint SubstrateOutput[3];
	SUBSTRATE_TOP_LAYER_TYPE SubstrateTopLayerData;

	// Optional SGGX distribution
	float4 SGGX;

	// Pixel Shader OutCoverage, only usable if PIXELSHADEROUTPUT_COVERAGE is 1
	uint Coverage;

	// Pixel Shader OutDepth
	float Depth;
};


#define SwapGeneric(T) void Swap(inout T A, inout T B) { T Temp = A; A = B; B = Temp; }
SwapGeneric(uint)
SwapGeneric(uint2)
SwapGeneric(uint3)
SwapGeneric(uint4)

SwapGeneric(float)
SwapGeneric(float2)
SwapGeneric(float3)
SwapGeneric(float4)

// ---------------------------------------------------- Global samplers.

// If GetGlobalSampler() was not implemented in Platform.ush, provide a default one from View uniform buffer.
#if !defined(GetGlobalSampler) && SUPPORTS_INDEPENDENT_SAMPLERS

// Implements UE4's Get global sampler.
// Filter={Point,Bilinear,Trilinear}
// WrapMode={Wrapped,Clamped}

#define GetGlobalSampler(Filter,WrapMode) \
	View.Shared##Filter##WrapMode##Sampler

#endif // GetGlobalSampler


#if SUPPORTS_INDEPENDENT_SAMPLERS

// Shortcuts for global samplers.
#define GlobalPointClampedSampler GetGlobalSampler(Point, Clamped)
#define GlobalPointWrappedSampler GetGlobalSampler(Point, Wrapped)
#define GlobalBilinearClampedSampler GetGlobalSampler(Bilinear, Clamped)
#define GlobalBilinearWrappedSampler GetGlobalSampler(Bilinear, Wrapped)
#define GlobalTrilinearClampedSampler GetGlobalSampler(Trilinear, Clamped)
#define GlobalTrilinearWrappedSampler GetGlobalSampler(Trilinear, Wrapped)

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5

float4 GatherDepth(Texture2D Texture, float2 UV)
{
	// using Gather: xyzw in counter clockwise order starting with the sample to the lower left of the queried location
	float4 DeviceZ = Texture.GatherRed(GlobalBilinearClampedSampler, UV);

	return float4(
		ConvertFromDeviceZ(DeviceZ.x),
		ConvertFromDeviceZ(DeviceZ.y),
		ConvertFromDeviceZ(DeviceZ.z),
		ConvertFromDeviceZ(DeviceZ.w));
}

#endif

#endif // SUPPORTS_INDEPENDENT_SAMPLERS

// ---------------------------------------------------- Ubershader

#ifndef UBERSHADER
#define UBERSHADER 0
#endif

#if UBERSHADER
#define IF_UBERSHADER(X) if(X)
#else
#define IF_UBERSHADER(X)
#endif