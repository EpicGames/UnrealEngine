// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CommonViewUniformBuffer.usf: Common view uniform buffer specifics
=============================================================================*/

#pragma once

/*
* @return tan(View.FieldOfViewWideAngles * .5)
*/
float2 GetTanHalfFieldOfView()
{
	return View.TanAndInvTanHalfFOV.xy;
}

float2 GetPrevTanHalfFieldOfView()
{
	return View.PrevTanAndInvTanHalfFOV.xy;
}

// might be used by Custom material expressions (still best to wrap the custom node in a material function)
// @return 1 / tan(View.FieldOfViewWideAngles * .5)
float2 GetCotanHalfFieldOfView()
{
	return View.TanAndInvTanHalfFOV.zw;
}

// might be used by Custom material expressions (still best to wrap the custom node in a material function)
// @return previous 1 / tan(View.FieldOfViewWideAngles * .5)
float2 GetPrevCotanHalfFieldOfView()
{
	return View.PrevTanAndInvTanHalfFOV.zw;
}

// Return the index of the frame.
uint GetPowerOfTwoModulatedFrameIndex(uint Pow2Modulus)
{
	// Bit masking of an uniform parameter is a scalar operation on modern hardware.
	return View.StateFrameIndex & uint(Pow2Modulus - 1);
}
