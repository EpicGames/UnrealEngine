// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "LensDistortion.ush"
#include "ScreenPass.ush"

#if PERMUTATION_NEARESTDEPTHNEIGHBOR 
	#include "SeparateTranslucency.ush"
#endif // PERMUTATION_NEARESTDEPTHNEIGHBOR


//------------------------------------------------------- PARAMETERS

Texture2D    SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D<float> LowResDepthTexture;
Texture2D<float> FullResDepthTexture;

Texture2D    SeparateTranslucencyPointTexture;
SamplerState SeparateTranslucencyPointSampler;
Texture2D    SeparateModulationPointTexture;
SamplerState SeparateModulationPointSampler;

Texture2D    SeparateTranslucencyBilinearTexture;
SamplerState SeparateTranslucencyBilinearSampler;
Texture2D    SeparateModulationBilinearTexture;
SamplerState SeparateModulationBilinearSampler;

Texture2D<float2> UndistortingDisplacementTexture;
SamplerState      UndistortingDisplacementSampler;

FScreenTransform ScreenPosToSceneColorUV;
FScreenTransform ScreenPosToSeparateTranslucencyUV;
FScreenTransform SeparateTranslucencyUVToViewportUV;
FScreenTransform ViewportUVToSeparateTranslucencyUV;

float2 SeparateTranslucencyUVMin;
float2 SeparateTranslucencyUVMax;
float2 SeparateTranslucencyExtentInverse;
uint bLensDistortion;
uint bPassthroughAlpha;

//------------------------------------------------------- ENTRY POINT

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float4 Debug = 0;

	float2 SceneColorUV = ApplyScreenTransform(SvPosition.xy, ScreenPosToSceneColorUV);
	float2 SeparateTranslucencyUV = ApplyScreenTransform(SvPosition.xy, ScreenPosToSeparateTranslucencyUV);
	BRANCH
	if (bLensDistortion)
	{
		float2 DistortedViewportUV = ApplyScreenTransform(SeparateTranslucencyUV, SeparateTranslucencyUVToViewportUV);
		float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);
		SeparateTranslucencyUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToSeparateTranslucencyUV);
	}

	float4 SceneColorSample = SceneColorTexture.SampleLevel(SceneColorSampler, SceneColorUV, 0);
	
	SeparateTranslucencyUV = clamp(SeparateTranslucencyUV, SeparateTranslucencyUVMin, SeparateTranslucencyUVMax);

#if PERMUTATION_NEARESTDEPTHNEIGHBOR 

	NearestDepthNeighborUpsamplingResult UpsampleResult = NearestDepthNeighborUpsampling(
		LowResDepthTexture,
		FullResDepthTexture,
		SvPosition.xy, SeparateTranslucencyUV, SeparateTranslucencyExtentInverse);

	float4 SeparateTranslucencySample = 0;
	float4 SeparateModulationSample = 0;
	
	float2 SeparateTranslucencySampleUV = clamp(UpsampleResult.UV, SeparateTranslucencyUVMin, SeparateTranslucencyUVMax);


	if (UpsampleResult.bUsePointSampler)
	{
		SeparateTranslucencySample	= SeparateTranslucencyPointTexture.SampleLevel(	SeparateTranslucencyPointSampler, SeparateTranslucencySampleUV, 0);
		SeparateModulationSample	= SeparateModulationPointTexture.SampleLevel(	SeparateModulationPointSampler, SeparateTranslucencySampleUV, 0);
	}
	else
	{
		SeparateTranslucencySample	= SeparateTranslucencyBilinearTexture.SampleLevel(	SeparateTranslucencyBilinearSampler, SeparateTranslucencySampleUV, 0);
		SeparateModulationSample	= SeparateModulationBilinearTexture.SampleLevel(	SeparateModulationBilinearSampler, SeparateTranslucencySampleUV, 0);
	}

#else // PERMUTATION_NEARESTDEPTHNEIGHBOR
	
	float4 SeparateTranslucencySample = SeparateTranslucencyBilinearTexture.SampleLevel(SeparateTranslucencyBilinearSampler, SeparateTranslucencyUV, 0);
	float4 SeparateModulationSample   = SeparateModulationBilinearTexture.SampleLevel(	SeparateModulationBilinearSampler, SeparateTranslucencyUV, 0);

#endif // PERMUTATION_NEARESTDEPTHNEIGHBOR

	// Final composition
	OutColor.rgb = SceneColorSample.rgb * SeparateTranslucencySample.a * SeparateModulationSample.rgb + SeparateTranslucencySample.rgb;
	
	BRANCH
	if (bPassthroughAlpha)
	{
		OutColor.a = SceneColorSample.a;
	}
	else
	{
		float GreyScaleModulateColorBackgroundVisibility = dot(SeparateModulationSample.rgb, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
		OutColor.a = SceneColorSample.a * SeparateTranslucencySample.a * GreyScaleModulateColorBackgroundVisibility;
	}
}

//------------------------------------------------------- PARAMETERS
#define VISIBILITY_COPY_FROM_SCENECOLOR 0
#define VISIBILITY_COPY_TO_SCENECOLOR	1

#ifndef VISIBILITY_COPY_TYPE
#define VISIBILITY_COPY_TYPE VISIBILITY_COPY_TO_SCENECOLOR
#endif

Texture2D    TranslucentHoldoutPointTexture;
SamplerState TranslucentHoldoutPointSampler;

//------------------------------------------------------- ENTRY POINT
void CopyBackgroundVisibilityPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 PixelPos = SvPosition.xy;
	
#if VISIBILITY_COPY_TYPE == VISIBILITY_COPY_FROM_SCENECOLOR
	float4 SceneColorWithBackgroundVisibilityInAlpha = TranslucentHoldoutPointTexture.Load(uint3(PixelPos, 0));
	OutColor.rgb = SceneColorWithBackgroundVisibilityInAlpha.a;
	OutColor.a = 1.0f;
#elif VISIBILITY_COPY_TYPE == VISIBILITY_COPY_TO_SCENECOLOR
	float4 BackgroundVisibilityWithAlpha = TranslucentHoldoutPointTexture.Load(uint3(PixelPos, 0));
	OutColor.rgb = 0.0f;
	OutColor.a =  saturate(BackgroundVisibilityWithAlpha.r);// We can copy any rgb channel.
#endif
}

//------------------------------------------------------- HOLDOUT COMPOSE WITH COVERAGE
//
// Holdout texture layout (set up by the translucent-holdout accumulation pass):
//   RGB = background visibility accumulated from translucent-holdout primitives
//   A   = remaining path throughput (initialized to 1; multiplied by (1 - Opacity) per layer,
//         so a value below 1 indicates a holdout primitive contributed at this pixel)

Texture2D    HoldoutTexture;
SamplerState HoldoutSampler;

void ComposeHoldoutWithCoveragePS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 HoldoutUV = ApplyScreenTransform(SvPosition.xy, ScreenPosToSeparateTranslucencyUV);
	HoldoutUV        = clamp(HoldoutUV, SeparateTranslucencyUVMin, SeparateTranslucencyUVMax);

	float4 HoldoutSample = HoldoutTexture.SampleLevel(HoldoutSampler, HoldoutUV, 0);
	float PathThroughput = saturate(HoldoutSample.a);

	// Leave scene-color alpha untouched at pixels where no holdout primitive contributed.
	if (PathThroughput >= 1.0f)
	{
		discard;
	}

	// RGB output is discarded by the CW_ALPHA blend the dispatch pairs with this shader.
	OutColor.rgb = 0.0f;
	OutColor.a   = saturate(HoldoutSample.r);
}
