// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
CompositeUIPixelShader.usf: Filter pixel shader source.
=============================================================================*/

#include "Common.ush"
#include "TonemapCommon.ush"
#include "ColorDeficiency.ush"

///////////////////////////////////////////////////////////////////////////////////////

#if APPLY_COLOR_DEFICIENCY
float ColorVisionDeficiencyType;
float ColorVisionDeficiencySeverity;
float bCorrectDeficiency;
float bSimulateCorrectionWithDeficiency;
#endif

///////////////////////////////////////////////////////////////////////////////////////
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
Texture2D<uint> UIWriteMaskTexture;
#endif

Texture2D UITexture;
SamplerState UISampler;

Texture2D SceneTexture;
SamplerState SceneSampler;

float UILevel;
float UILuminance;
int UICompositeEOTF;
float UICompositeEOTFGamma;
int UICompositeBlend;
float UICompositeBlendGamma;
float UICompositeBlendInvGamma;

// Apply "EOTF" on SDR UI for HDR composition.
float3 LinearizeColor(float3 EncodedColor)
{
	return (UICompositeEOTF == 0) ? pow(EncodedColor, UICompositeEOTFGamma) : sRGBToLinear(EncodedColor);
}

float3 ComposeUIAndScene(float3 SceneColor, float4 InUIColor, float InUILevel)
{
	BRANCH
	if (InUIColor.a > 0.f)
	{
		InUIColor.rgb *= InUILevel;

		BRANCH
		if (InUIColor.a < 1.f)
		{
			BRANCH
			if (UICompositeBlend)
			{
				// Clamp gamut to sRGB as extended gamut colors bleeding into the UI can look funny
				SceneColor = max(SceneColor, 0.f);

				// Tonemap HDR under transparent UI with a simple Reinhard to the max luminance of the UI
				// This prevents HDR bleeding through and destroying UI legibility
				// Rec2020 coefficients to compute luminance
				float KR = 0.2627, KG = 0.678, KB = 0.0593;
				float Luminance = dot(SceneColor, half3(KR, KG, KB)) / InUILevel;
				float OutL = 1.f / (Luminance + 1.f);

				// Ease out remapping to avoid hard transitions where UI is near zero opacity
				SceneColor *= lerp(1.f, OutL * InUILevel, InUIColor.a);

				// Composite in linear blend space
				SceneColor = lerp(SceneColor, InUIColor.rgb, InUIColor.a);
			}
			else
			{
				// Convert inputs to gamma blend space
				InUIColor.rgb = pow(saturate(InUIColor.rgb / UE_PQ_WHITE_NITS), UICompositeBlendInvGamma);
				SceneColor = pow(saturate(SceneColor / UE_PQ_WHITE_NITS), UICompositeBlendInvGamma);

				// Composite in gamma blend space
				SceneColor = lerp(SceneColor, InUIColor.rgb, InUIColor.a);

				// Convert result to nits
				SceneColor = pow(SceneColor, UICompositeBlendGamma) * UE_PQ_WHITE_NITS;
			}
		}
		else
		{
			SceneColor = InUIColor.rgb;
		}
	}

	return SceneColor;
}

float3 ApplyColorDeficiency(float3 Rec2020Color)
{
#if APPLY_COLOR_DEFICIENCY
	const float3x3 Rec2020_2_sRGB = mul(XYZ_2_sRGB_MAT, Rec2020_2_XYZ_MAT);
	const float3x3 sRGB_2_Rec2020 = mul(XYZ_2_Rec2020_MAT, sRGB_2_XYZ_MAT);
	float3 SRGBColor = mul(Rec2020_2_sRGB, Rec2020Color.xyz);
	SRGBColor = ColorDeficiency(SRGBColor, ColorVisionDeficiencyType, ColorVisionDeficiencySeverity, bCorrectDeficiency, bSimulateCorrectionWithDeficiency);
	Rec2020Color = mul(sRGB_2_Rec2020, SRGBColor);
#endif

	return Rec2020Color;
}

// About sRGB_2_Rec2020 in ComputeHDRUIColor/ComputeHDRSceneColor: to get consistent blending between UI and scene color, regardless of the swapchain format choice, we perform blending in the rec2020 color space, and
// move back to the output device space. Rec2020 is chosen over ScRGB because this is the way blending is currently being done in engine mode, and contains only positive values

float4 ComputeHDRUIColor(float2 InputUV)
{
	const float3x3 sRGB_2_Rec2020 = mul(XYZ_2_Rec2020_MAT, sRGB_2_XYZ_MAT);
	float4 UIColor = Texture2DSample(UITexture, UISampler, InputUV);

	// Undo premultiplied alpha to apply in HDR blend space
	UIColor.rgb = (UIColor.a > 0.) ? (UIColor.rgb / UIColor.a) : 0.;

	UIColor.rgb = LinearizeColor(UIColor.rgb);
	UIColor.rgb = mul(sRGB_2_Rec2020, UIColor.rgb) * UILuminance;
	return UIColor;
}

float3 ComputeHDRSceneColor(float3 SceneColor)
{
#if !SCRGB_ENCODING
	SceneColor = ST2084ToLinear(SceneColor);
#else
	const float3x3 sRGB_2_Rec2020 = mul(XYZ_2_Rec2020_MAT, sRGB_2_XYZ_MAT);
	SceneColor = mul(sRGB_2_Rec2020, SceneColor) * UE_SCRGB_WHITE_NITS;
#endif
	return SceneColor;
}

struct FCompositeVertexInput
{
	noperspective float4 UV : TEXCOORD0;
	float4 Position : SV_POSITION;
};

void Main(
	FCompositeVertexInput Input,
	out float4 OutColor : SV_Target0
	)
{
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	uint CompositeUIMask = DecodeRTWriteMask(uint2(Input.Position.xy), UIWriteMaskTexture, 1);
	// in case of color deficiency, we will need to update the whole texture, not just the regions that were touched by the UI
#if (APPLY_COLOR_DEFICIENCY == 0)
	BRANCH
	if (CompositeUIMask == 0)
	{
		discard;
	}
#endif
#endif

	float4 UIColor = ComputeHDRUIColor(Input.UV.xy);
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	// If the UI texture has a mask of 0, this means the UI color needs to be forced to 0 as it may contain uninitialized data
	if (CompositeUIMask == 0)
	{
		UIColor = 0;
	}
#endif

	float3 SceneColor = Texture2DSample(SceneTexture, SceneSampler, Input.UV.xy).xyz;
	SceneColor = ComputeHDRSceneColor(SceneColor);
	OutColor.xyz = ComposeUIAndScene(SceneColor.xyz, UIColor, UILevel);

	OutColor.rgb = ApplyColorDeficiency(OutColor.rgb);

	// This is not compatible when alpha is preserved through post-processing
	OutColor.w = UIColor.w;

#if !SCRGB_ENCODING
	// Linear -> PQ
	OutColor.xyz = LinearToST2084(OutColor.xyz);
#else
	const float3x3 Rec2020_2_sRGB = mul(XYZ_2_sRGB_MAT, Rec2020_2_XYZ_MAT);
	OutColor.xyz = mul(Rec2020_2_sRGB, OutColor.xyz / UE_SCRGB_WHITE_NITS);
#endif
}

#if USE_COMPUTE_FOR_COMPOSITION
float4 SceneTextureDimensions;
RWTexture2D<float4> RWSceneTexture;

[numthreads(NUM_THREADS_PER_GROUP, NUM_THREADS_PER_GROUP, 1)]
void CompositeUICS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId.xy >= (uint2)SceneTextureDimensions.xy))
	{
		return;
	}

	float2 InputPosition = DispatchThreadId.xy + float2(0.5, 0.5);
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	uint CompositeUIMask = DecodeRTWriteMask(DispatchThreadId.xy, UIWriteMaskTexture, 1);
	// in case of color deficiency, we will need to update the whole texture, not just the regions that were touched by the UI
#if (APPLY_COLOR_DEFICIENCY == 0)
	BRANCH
	if (CompositeUIMask == 0)
	{
		return;
	}
#endif
#endif

	float2 InputUV = InputPosition * SceneTextureDimensions.zw;

	float4 UIColor = ComputeHDRUIColor(InputUV);
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	// If the UI texture has a mask of 0, this means the UI color needs to be forced to 0 as it may contain uninitialized data
	if (CompositeUIMask == 0)
	{
		UIColor = 0;
	}
#endif


	float3 SceneColor = RWSceneTexture[DispatchThreadId.xy].rgb;
	SceneColor = ComputeHDRSceneColor(SceneColor);
	float4 OutColor = 0;
	OutColor.xyz = ComposeUIAndScene(SceneColor.xyz, UIColor, UILevel);
	
	// This is not compatible when alpha is preserved through post-processing
	OutColor.w = UIColor.w;

	OutColor.rgb = ApplyColorDeficiency(OutColor.rgb);

#if !SCRGB_ENCODING
	// Linear -> PQ
	OutColor.xyz = LinearToST2084(OutColor.xyz);
#else
	const float3x3 Rec2020_2_sRGB = mul(XYZ_2_sRGB_MAT, Rec2020_2_XYZ_MAT);
	OutColor.xyz = mul(Rec2020_2_sRGB, OutColor.xyz / UE_SCRGB_WHITE_NITS);
#endif

	RWSceneTexture[DispatchThreadId.xy] = OutColor;
}

#endif

#if COMPOSITE_UI_FOR_BLUR_PS

float2 UITextureSize;

void CompositeUIForBlur(FScreenVertexOutput Input, out float4 OutColor : SV_Target0)
{
	float4 UIColor = ComputeHDRUIColor(Input.UV);
#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	uint2 RTWriteMaskInputPosition = uint2(Input.UV * UITextureSize);
	uint CompositeUIMask = DecodeRTWriteMask(RTWriteMaskInputPosition, UIWriteMaskTexture, 1);
	BRANCH
	if (CompositeUIMask == 0)
	{
		UIColor = float4(0,0,0,0);
	}
#endif

	float3 SceneColor = Texture2DSample(SceneTexture, UISampler, Input.UV).xyz;
	SceneColor = ComputeHDRSceneColor(SceneColor);
	OutColor.xyz = ComposeUIAndScene(SceneColor.xyz, UIColor, UILevel) / UE_SCRGB_WHITE_NITS;
	OutColor.a = 1;
}

#endif
