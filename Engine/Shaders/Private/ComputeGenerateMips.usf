// Copyright Epic Games, Inc. All Rights Reserved.

/*==================================================================================================
	GenericGenerateMips.usf: Standard code for generating mips via the compute shader in realtime
===================================================================================================*/

#pragma once
#include "Common.ush"
#include "GammaCorrectionCommon.ush"

Texture2D MipInSRV;
SamplerState MipSampler;

#if GENMIPS_COMPUTE

float2 TexelSize;
#if GENMIPS_SRGB
RWTexture2D<half4> MipOutUAV;
#else
RWTexture2D<float4> MipOutUAV;
#endif

[numthreads(8, 8, 1)]
void MainCS(uint3 DT_ID : SV_DispatchThreadID)
{
    float2 UV = TexelSize * (DT_ID.xy + 0.5f);

#if GENMIPS_SRGB
    half4 outColor = MipInSRV.SampleLevel(MipSampler, UV, 0);
    outColor = half4(LinearToSrgb(outColor.xyz), outColor.w);
#else
    float4 outColor = MipInSRV.SampleLevel(MipSampler, UV, 0);
#endif

#if GENMIPS_SWIZZLE
    MipOutUAV[DT_ID.xy] = outColor.zyxw;
#else
    MipOutUAV[DT_ID.xy] = outColor;
#endif
}

uint2 TextureSize;
uint Offset;
uint NumMips;
Buffer<uint> ConditionBuffer;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;

[numthreads(8,1,1)]
void BuildIndirectDispatchArgsCS(uint DT_ID : SV_DispatchThreadID)
{
	if (DT_ID < NumMips)
	{
		uint Condition = ConditionBuffer[Offset];
		
		// Generate mipmaps only when Condition > 0
		const bool bShouldDispatchGenerateMips = (Condition > 0);

		if (bShouldDispatchGenerateMips)
		{

			uint2 DestTextureSize = uint2(
				max(TextureSize.x >> (DT_ID + 1u), 1u),
				max(TextureSize.y >> (DT_ID + 1u), 1u));

			DestTextureSize = (DestTextureSize + 8 - 1) / 8;
			WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, DT_ID, DestTextureSize.x, DestTextureSize.y, 1);
		}
		else
		{
			WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, DT_ID, 0, 0, 0);
		}
	}
}

#else // !GENMIPS_COMPUTE

float Level;

void MainVS(in float4 InPosition : ATTRIBUTE0, in float2 InUV : ATTRIBUTE1,	out float4 OutPosition : SV_POSITION, out float2 OutUV : TEXCOORD0)
{
	OutPosition = InPosition;
	OutPosition.xy = -1.0f + 2.0f * InPosition.xy;
	OutPosition.xy *= float2( 1, -1 );
	OutUV = InUV;
}

void MainPS(float4 InPosition : SV_POSITION,	float2 InUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = MipInSRV.SampleLevel(MipSampler, InUV, Level);
}

#endif // GENMIPS_COMPUTE