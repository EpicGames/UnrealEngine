// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ComputeShaderOutputCommon.ush: To allow CS input/output passed into functions 
through a single struct, allowing for a more readable code
(less #ifdefs, reducing the boolean hell)
=============================================================================*/

COMPILER_ALLOW_CS_DERIVATIVES

#if IS_NANITE_PASS
MAX_OCCUPANCY
DISABLE_TARGET_OCCUPANCY_WARNING
#endif

#include "ShaderOutputCommon.ush"
#include "GammaCorrectionCommon.ush"
#include "VariableRateShading/VRSShadingRateCommon.ush"
#include "Nanite/NaniteShadeCommon.ush"

#if WORKGRAPH_NODE
#include "ShaderBundleWorkGraphCommon.ush"
#endif

#include "/Engine/Public/RootConstants.ush"

uint GetShadingGroup()			{ return GetRootConstant0(); }
uint GetShadingGroupCount()		{ return GetRootConstant1(); }
uint GetShadingBinCount()		{ return GetRootConstant2(); }
bool GetQuadBinning()			{ return (GetRootConstant3() & 1u) != 0u; }
bool GetHighPrecision()			{ return (GetRootConstant3() & 2u) != 0u; }


uint GetShadingBinMetaOffset()
{
	return GetShadingGroupCount() * 16;
}

uint GetShadingBinDataOffset()
{
	return GetShadingBinMetaOffset() + GetShadingBinCount() * NANITE_SHADING_BIN_META_BYTES;
}

#define SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER (SUBSTRATE_OPAQUE_DEFERRED && SUBSTRATE_GBUFFER_FORMAT==1)

#if SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER
	#if SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT != 3
		#error Substrate SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT has been updated but not the uint MRTs
	#endif
	#if PIXELSHADEROUTPUT_MRT4
		#error Substrate cannot map to such a case
	#endif
	#if SUBSTRATE_FIRST_MRT_INDEX > 4
		#error Substrate doesn't currently handle more than 4 base GBuffer MRTs
	#endif
#endif

FPixelShaderOut ShadePixel(const float2 SVPositionXY, uint QuadIndex, uint QuadPixelWriteMask, uint2 QuadPixelPos)
{
	// Note: Driven by bHighPrecisionGBuffers in GBufferInfo.cpp
	const bool bHighPrecision = GetHighPrecision();

#if PIXELSHADEROUTPUT_INTERPOLANTS || PIXELSHADEROUTPUT_BASEPASS
#if IS_NANITE_PASS
	FNaniteFullscreenVSToPS NaniteInterpolants = (FNaniteFullscreenVSToPS)0;
	NaniteInterpolants.TileIndex = QuadIndex;
#else
	FVertexFactoryInterpolantsVSToPS Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
#endif
#endif

	const float4 SvPosition = float4(SVPositionXY, 0.0f, 1.0f);
	uint EyeIndex = 0;

#if IS_NANITE_PASS && (PIXELSHADEROUTPUT_INTERPOLANTS || PIXELSHADEROUTPUT_BASEPASS)
	FVertexFactoryInterpolantsVSToPS Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
	Interpolants.PixelPos = int2(SVPositionXY);
	Interpolants.ViewIndex = NaniteInterpolants.ViewIndex; // TODO: NANITE_MATERIAL_MULTIVIEW

#if INSTANCED_STEREO
	// Revisit if we need to support > 1 instanced view or non side-by-side views
	Interpolants.EyeIndex = (SvPosition.x >= (View.ViewRectMin.x + View.ViewSizeAndInvSize.x)) ? 1 : 0;
	StereoSetupCS(Interpolants.EyeIndex);
	EyeIndex = Interpolants.EyeIndex;
#else
	StereoSetupCS();
#endif
#endif

	FPixelShaderIn PixelShaderIn = (FPixelShaderIn)0;
	FPixelShaderOut PixelShaderOut = (FPixelShaderOut)0;

	PixelShaderIn.SvPosition = SvPosition;

	// Nanite does not support OPTIONAL_IsFrontFace, Instead, Nanite determines this in GetMaterialPixelParameters().
	PixelShaderIn.bIsFrontFace = false;

#if PIXELSHADEROUTPUT_BASEPASS
	FBasePassInterpolantsVSToPS BasePassInterpolants = (FBasePassInterpolantsVSToPS)0;
	FPixelShaderInOut_MainPS(Interpolants, BasePassInterpolants, PixelShaderIn, PixelShaderOut, EyeIndex, QuadPixelWriteMask, QuadPixelPos, true/*QuadShadingEnabled*/);
#endif

#if !SUBSTRATE_ENABLED || (SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT == 0)
	if (!bHighPrecision)
	{
		PixelShaderOut.MRT[3] = float4(LinearToSrgb(PixelShaderOut.MRT[3].rgb), PixelShaderOut.MRT[3].a); // BaseColor is sRGB
	}
#endif

	return PixelShaderOut;
}

#if SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER

void SubstrateExport(uint2 PixelPos, FPixelShaderOut ShadedPixel, uint Index)
{
	ComputeShadingOutputs.OutTargets[uint3(PixelPos, Index)] = ShadedPixel.SubstrateOutput[Index];
}

void SubstrateExport(uint2 PixelPos, SUBSTRATE_TOP_LAYER_TYPE TopLayerData)
{
	ComputeShadingOutputs.OutTopLayerTarget[PixelPos] = TopLayerData;
}

#endif

void ExportPixel(const uint2 PixelPos, FPixelShaderOut ShadedPixel)
{
#if PIXELSHADEROUTPUT_COVERAGE || PIXELSHADEROUTPUT_A2C
	// TODO: OutCoverage = PixelShaderOut.Coverage;
#endif 

#if OUTPUT_PIXEL_DEPTH_OFFSET
	// TODO: OutDepth = PixelShaderOut.Depth;
#endif 

#if PIXELSHADEROUTPUT_MRT0
	ComputeShadingOutputs.OutTarget0[PixelPos] = ShadedPixel.MRT[0];
#endif

#if PIXELSHADEROUTPUT_MRT1
	ComputeShadingOutputs.OutTarget1[PixelPos] = ShadedPixel.MRT[1];
#endif

#if PIXELSHADEROUTPUT_MRT2
	ComputeShadingOutputs.OutTarget2[PixelPos] = ShadedPixel.MRT[2];
#endif

#if PIXELSHADEROUTPUT_MRT3
	ComputeShadingOutputs.OutTarget3[PixelPos] = ShadedPixel.MRT[3];
#endif

#if SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER

	// In this case, here is the gbuffer pattern
	// MRT0 is pixel color
	// MRT1 is velocity if enabled or precomputed shadow if velocity if disabled and precomputed shadow enabled
	// MRT2 is precomputed shadow if both velocity and prec shadow are enabled.
	// After, Substrate top layer data appended. Remaining Substrate outputs are in the 2d array UAV

	// Export Substrate data
	// * We unconditionally write all required MRTs. Because, for instance, simple/fast material encoding can have F0=Diffuse=Black making the second uint be 0 when we DO want to write that data. Same for other encoding types.
	// * SubstrateTopLayerData is also always exported for the same reason
	UNROLL 
	for (uint LayerIt=0; LayerIt<SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT; ++LayerIt)
	{
		SubstrateExport(PixelPos, ShadedPixel, LayerIt);
	}
	SubstrateExport(PixelPos, ShadedPixel.SubstrateTopLayerData);

#else // SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER

#if PIXELSHADEROUTPUT_MRT4
	ComputeShadingOutputs.OutTarget4[PixelPos] = ShadedPixel.MRT[4];
#endif

#if PIXELSHADEROUTPUT_MRT5
	ComputeShadingOutputs.OutTarget5[PixelPos] = ShadedPixel.MRT[5];
#endif

#if PIXELSHADEROUTPUT_MRT6
	ComputeShadingOutputs.OutTarget6[PixelPos] = ShadedPixel.MRT[6];
#endif

#if PIXELSHADEROUTPUT_MRT7
	ComputeShadingOutputs.OutTarget7[PixelPos] = ShadedPixel.MRT[7];
#endif

#if NANITE_USE_ANALYTIC_SGGX && IS_NANITE_PASS && PIXELSHADEROUTPUT_BASEPASS && MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	ComputeShadingOutputs.OutSGGX[PixelPos] = ShadedPixel.SGGX;
#endif

#endif // SUBSTRATE_OPAQUE_DEFERRED_ADAPTATIVE_GBUFFER
}

#define VIS_HELPER_LANES 0

void ProcessPixel(uint ShadingBin, const uint2 PixelPos, const float2 SVPositionXY, uint QuadIndex, uint DispatchIndex, uint PixelWriteMask, uint HelperLaneCount)
{
	// All lanes shade (regardless of export - so ddx/ddy are valid)
	FPixelShaderOut ShadedPixel = ShadePixel(SVPositionXY, QuadIndex, PixelWriteMask, PixelPos /*QuadPixelPos*/);

#if VIS_HELPER_LANES
	ShadedPixel.MRT[3].rgb = ColorMapTurbo(float(HelperLaneCount) / 4.0f);
#elif 0
	ShadedPixel.MRT[3].rgb = ColorMapTurbo(50.0);
#elif 0
	// Coherency vis
	float R =  (DispatchIndex & 0xFFu) / 255.0f;
	float G = ((DispatchIndex & 0xFF00u) >> 8u) / 255.0f;
	float B = ((DispatchIndex & 0xFF0000u) >> 16u) / 255.0f;
	ShadedPixel.MRT[3].rgb = float3(R, G, B);
#elif 0
	ShadedPixel.MRT[3].rgb = IntToColor(ShadingBin);
#elif 0
	ShadedPixel.MRT[3].rgb = IntToColor(QuadIndex);
#elif 0
	ShadedPixel.MRT[3].rgb = VisualizeShadingRate(ShadingRate).rgb;
#endif

	// Disable helper lanes from final export
	BRANCH
	if (PixelWriteMask & 1u)
	{
		ExportPixel(PixelPos, ShadedPixel);
	}

	BRANCH
	if (PixelWriteMask & 2u) // Copy H
	{
		ExportPixel(PixelPos + uint2(1, 0), ShadedPixel);
	}

	BRANCH
	if (PixelWriteMask & 4u) // Copy V
	{
		ExportPixel(PixelPos + uint2(0, 1), ShadedPixel);
	}

	BRANCH
	if (PixelWriteMask & 8u) // Copy D
	{
		ExportPixel(PixelPos + uint2(1, 1), ShadedPixel);
	}
}

FNaniteShadingBinMeta GetShadingBinMeta(uint ShadingBin)
{
	return NaniteShading.ShadingBinData.Load<FNaniteShadingBinMeta>(GetShadingBinMetaOffset() + ShadingBin * NANITE_SHADING_BIN_META_BYTES);
}

#if WORKGRAPH_NODE
[Shader("node")]
[NodeLaunch("broadcasting")]
[NodeMaxDispatchGrid(65535,1,1)]
#endif
[numthreads(COMPUTE_MATERIAL_GROUP_SIZE, 1, 1)]
void MainCS(
	uint ThreadIndex : SV_GroupIndex, 
	uint GroupID : SV_GroupID
#if WORKGRAPH_NODE
	, DispatchNodeInputRecord<FShaderBundleNodeRecord> InputRecord
#endif
)
{
	
#if UE_BINDLESS_ENABLED && SUPPORTS_BINDLESS_UNIFORM_BUFFER
	const uint ShadingGroup = GetShadingGroup();
	
	const uint ThreadGroupOffset = NaniteShading.ShadingBinData.Load4(ShadingGroup * 16).x;
	const uint ShadingBin = NaniteShading.ThreadGroupData[ThreadGroupOffset + GroupID];
	
	const FNaniteShadingBinMeta ShadingBinMeta = GetShadingBinMeta(ShadingBin);

	Material = GetMaterialConstants(WaveReadLaneFirst(ShadingBinMeta.MaterialCBIndex));

	const uint PixelIndex = ((GroupID - ShadingBinMeta.FirstThreadGroup) * COMPUTE_MATERIAL_GROUP_SIZE) + ThreadIndex;
#else
	const uint ShadingBin = GetShadingGroup();	// Shading Group and Bin are the same when bindless is disabled.
	
	const FNaniteShadingBinMeta ShadingBinMeta = GetShadingBinMeta(ShadingBin);
	
	const uint PixelIndex = (GroupID * COMPUTE_MATERIAL_GROUP_SIZE) + ThreadIndex;
#endif

	const bool bQuadBinning = GetQuadBinning();
	const uint ElementCount = ShadingBinMeta.ElementCount;
	const uint ElementIndex = bQuadBinning ? (PixelIndex >> 2) : PixelIndex;
	const uint DataByteOffset = GetShadingBinDataOffset();

	BRANCH
	if (ElementIndex >= ElementCount)
	{
		return;
	}

	uint2 PixelPos;
	uint2 VRSShift;
	uint PixelWriteMask;
	uint HelperLaneCount;
	
	BRANCH
	if (bQuadBinning)
	{
		const uint2 PackedElement = NaniteShading.ShadingBinData.Load2(DataByteOffset + (ShadingBinMeta.RangeStart * 4 + ElementIndex * 8));

		const uint CornerIndex = (ThreadIndex & 3u);

		const uint2 TopLeft = uint2(BitFieldExtractU32(PackedElement.x, 14, 0), BitFieldExtractU32(PackedElement.x, 14, 14));
		VRSShift			= uint2(BitFieldExtractU32(PackedElement.x, 1, 28), BitFieldExtractU32(PackedElement.x, 1, 29));
		PixelWriteMask		= BitFieldExtractU32(PackedElement.y, 4, CornerIndex * 4u);

		PixelPos			= TopLeft + (uint2(CornerIndex & 1u, CornerIndex >> 1u) << VRSShift);
		uint NumActiveLanes = ((PackedElement.y & 0xF) != 0) + ((PackedElement.y & 0xF0) != 0) + ((PackedElement.y & 0xF00) != 0) + ((PackedElement.y & 0xF000) != 0);
		HelperLaneCount		= (VIS_HELPER_LANES && bQuadBinning) ? (4u - NumActiveLanes) : 0u;
	}
	else
	{
		// See format description in PackShadingPixel(...)
		uint PackedElement = NaniteShading.ShadingBinData.Load(DataByteOffset + (ShadingBinMeta.RangeStart * 4 + ElementIndex * 4));
		
		VRSShift = uint2(BitFieldExtractU32(PackedElement, 1, 30), BitFieldExtractU32(PackedElement, 1, 31));

		PackedElement = (PackedElement & 0x3FFFFFFFu);		// Clear VRSShift bits so they don't become part of PixelWriteMask
		PackedElement = PackedElement << VRSShift.x;		// Add implicit low zero bit if this is a VRS coarse offset (13 -> 14 bit value)
		PixelPos.x = PackedElement & 0x3fff;				// 14 bit result
		
		PackedElement = PackedElement >> 14;				// Advance to next packed data member
		PackedElement = PackedElement << VRSShift.y;		// Add implicit low zero bit if this is a VRS coarse offset (13 -> 14 bit value)
		PixelPos.y = PackedElement & 0x3fff;				// 14 bit result
		
		PixelWriteMask = PackedElement >> 14;				// Advance to next packed data member, and store it out

		HelperLaneCount = 0;
	}

#if 0
	// The VRS shift into coarse pixel center causes mismatches between
	// SvPosition used for base pass shading vs raster. Base pass shading
	// could follow the barycentrics approached used by raster to match,
	// but this would be a pretty significant cost compared to how it currently
	// works. While ignoring the shift does not exactly match HW VRS,
	// it doesn't seem super important outside of a direct A/B test.
	const float2 SVPositionXY = PixelPos + ((1u << VRSShift) * 0.5f);
#elif 0
	// Always shade as top-left. For VRS this could be outside the rasterized triangle.
	const float2 SVPositionXY = PixelPos + 0.5f;
#else
	// Centroid-like sampling. Move the sampling point to the center of the first live pixel.
	const uint WriteMaskFirstIndex = PixelWriteMask ? firstbitlow(PixelWriteMask) : 0u;	// PixelWriteMask can be 0 in quad mode.
	const float2 SVPositionXY = PixelPos + int2(WriteMaskFirstIndex & 1, (WriteMaskFirstIndex >> 1) & 1) + 0.5f;
#endif

#if WORKGRAPH_NODE
	// Fix for GPU hang. Without this a lack of out of bounds checking in Work Graph shaders causes out of bounds access of VisBuffer.
	// Note that derivatives can be incorrect on odd edge pixels, and VRS edge pixels may get rejected.
	// But that's also true in general for the standard compute path.
	BRANCH
	if (any(SVPositionXY > float2(View.ViewRectMinAndSize.xy + View.ViewRectMinAndSize.zw)))
	{
		return;
	}
#endif

	ProcessPixel(ShadingBin, PixelPos, SVPositionXY, ElementIndex, PixelIndex, PixelWriteMask, HelperLaneCount);
}