// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Note: this value must match WrappedGroupStride in RenderGraphUtils.h
#define WRAPPED_GROUP_STRIDE (128U)

/**
 * Convert wrapped (when overflowing 1D dimension) SV_GroupID created by using FComputeShaderUtils::GetGroupCountWrapped to a linear group id.
 * NOTE: when using this you need to check the group count against whatever max you have as the wrapping means there may be indices larger than the requested (unlike when you use a regular 1D dispatch).
 */
uint GetUnWrappedDispatchGroupId(uint3 GroupId /*SV_GroupID*/)
{
	return GroupId.x + (GroupId.z * WRAPPED_GROUP_STRIDE + GroupId.y) * WRAPPED_GROUP_STRIDE;
}

/**
 * Calculate linear dispatch thread id from a wrapped GroupId : SV_GroupID (created by using FComputeShaderUtils::GetGroupCountWrapped) and 
 * a linear GroupThreadIndex : SV_GroupIndex. 
 */
uint GetUnWrappedDispatchThreadId(uint3 GroupId /*SV_GroupID*/, uint GroupThreadIndex /*SV_GroupIndex*/, uint ThreadGroupSize)
{
	return GetUnWrappedDispatchGroupId(GroupId) * ThreadGroupSize + GroupThreadIndex;
}

/**
* Wrapping number of groups to Y and Z dimension if X group count overflows DimensionLimit.
* Calculate the linear group index as:
*  uint LinearGroupId = GroupId.X + (GroupId.Z * WRAPPED_GROUP_STRIDE + GroupId.Y) * WRAPPED_GROUP_STRIDE;
* Note that you must use an early out because LinearGroupId may be larger than the ideal due to wrapping.
*/
uint3 GetGroupCountWrapped(const uint TargetGroupCount, uint2 DimensionLimit)
{
	uint3 GroupCount = uint3(TargetGroupCount, 1, 1);

	if (GroupCount.x > DimensionLimit.x)
	{
		GroupCount.y = (GroupCount.x + WRAPPED_GROUP_STRIDE - 1u) / WRAPPED_GROUP_STRIDE;
		GroupCount.x = WRAPPED_GROUP_STRIDE;
	}

	if (GroupCount.y > DimensionLimit.y)
	{
		GroupCount.z = (GroupCount.y + WRAPPED_GROUP_STRIDE - 1u) / WRAPPED_GROUP_STRIDE;
		GroupCount.y = WRAPPED_GROUP_STRIDE;
	}

	return GroupCount;
}
