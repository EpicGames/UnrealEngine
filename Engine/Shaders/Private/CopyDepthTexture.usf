// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CopyDepthTexture.usf
=============================================================================*/

#include "Common.ush"

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, MSAA_SAMPLE_COUNT> DepthTextureMS;
#else
Texture2D<float> DepthTexture;
#endif

float CopyDepthPS(
	float4 Position : SV_POSITION
#if MSAA_SAMPLE_COUNT > 1
	, uint SampleIndex : SV_SampleIndex
#endif
) : SV_DEPTH
{
#if MSAA_SAMPLE_COUNT > 1
	return DepthTextureMS.Load(int2(Position.xy), SampleIndex).x;
#else
	return DepthTexture.Load(int3(Position.xy, 0)).x;
#endif
}