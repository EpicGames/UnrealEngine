// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CopyDepthTextureCS.usf
=============================================================================*/

#include "Common.ush"
#include "HTileEncoding.ush"

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 8
#endif

Texture2D SceneDepthTexture;

RWTexture2D<float> RWDepthTexture;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void CopyDepthCS(
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float DeviceZ = SceneDepthTexture[DispatchThreadId + View.ViewRectMinAndSize.xy].x;
	RWDepthTexture[DispatchThreadId + View.ViewRectMinAndSize.xy] = DeviceZ;
}

#if PLATFORM_SUPPORTS_HTILE_LOOKUP

#if !COMPILER_SUPPORTS_WAVE_MINMAX
#error "CS depth copy with HTILE requires wave min/max support"
#endif

uint PlatformConfig;
uint PixelsWide;
uint4 ViewRect;

Texture2D<float> SrcDepth;
RWTexture2D<float> DstDepth;
RWStructuredBuffer<uint> DstHTile;

[numthreads(HTILE_PIXELS_PER_TILE_WIDE, HTILE_PIXELS_PER_TILE_TALL, 1)]
void CopyDepthAndBuildHTileCS(uint2 GroupId : SV_GroupID, uint ThreadIndex : SV_GroupIndex)
{
	const uint2 PixelLocalPos = SwizzleThreadIndex(ThreadIndex & 63u);
	const uint2 PixelPos = ((GroupId << 3) | PixelLocalPos) + ViewRect.xy;

	float DepthValue = SrcDepth[PixelPos];

	// Write uncompressed depth
	DstDepth[PixelPos] = DepthValue;

	// Compute tile min/max across the 8x8 HTILE tile
	float TileMinDepth = WaveActiveMin(DepthValue);
	float TileMaxDepth = WaveActiveMax(DepthValue);

	if (WaveIsFirstLane())
	{
		uint TileIndex = ToScalarMemory(ComputeTileOffset(PixelPos, PixelsWide, PlatformConfig));

		BRANCH
			if (IsHiStencilTileConfig(PlatformConfig))
			{
				DstHTile[TileIndex] = EncodeTileStencilZBaseDelta(
					TileMinDepth,
					TileMaxDepth,
					HTILE_DEFAULT_XX,
					HTILE_DEFAULT_SMEM,
					HTILE_DEFAULT_SR0,
					HTILE_DEFAULT_SR1,
					IsZCompareGE(PlatformConfig)
				);
			}
			else
			{
				DstHTile[TileIndex] = EncodeTileMinMaxDepth(TileMinDepth, TileMaxDepth);
			}
	}
}

#else

[numthreads(8, 8, 1)]
void CopyDepthAndBuildHTileCS(uint3 PixelPos : SV_DispatchThreadID)
{
}

#endif