// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CopyShadowMaps.usf
=============================================================================*/

#include "Common.ush"

Texture2D ShadowDepthTexture;
SamplerState ShadowDepthSampler;

void Copy2DDepthPS(
	FScreenVertexOutput Input,
	out float OutDepth : SV_DEPTH,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = 0;
	OutDepth = ShadowDepthTexture.SampleLevel(ShadowDepthSampler, Input.UV, 0).x;
}

struct FCubeCopyGSToPS
{
	FScreenVertexOutput Vertex;
	nointerpolation uint FaceIndex : TEXCOORD1;

	/** Controls which of the cube map faces to rasterize the primitive to, only the value from the first vertex is used. */
	nointerpolation uint RTIndex : SV_RenderTargetArrayIndex;
};
 
/** Allocate space for cloning to all 6 faces. */
[maxvertexcount(18)]
void CopyCubeDepthGS(triangle FScreenVertexOutput Input[3], inout TriangleStream<FCubeCopyGSToPS> OutStream)
{
	UNROLL
	// Clone the triangle to each face
    for (uint CubeFaceIndex = 0; CubeFaceIndex < 6; CubeFaceIndex++)
    {
		FCubeCopyGSToPS Output;
		Output.RTIndex = CubeFaceIndex;
		Output.FaceIndex = CubeFaceIndex;

		UNROLL
		for (uint VertexIndex = 0; VertexIndex < 3; VertexIndex++)
		{
			Output.Vertex = Input[VertexIndex];
			OutStream.Append(Output);
		}
		OutStream.RestartStrip();
	}
}

float3 GetCubemapVector(float2 ScaledUVs, uint CubeFace)
{
	float3 CubeCoordinates;

	if (CubeFace == 0)
	{
		CubeCoordinates = float3(1, -ScaledUVs.y, -ScaledUVs.x);
	}
	else if (CubeFace == 1)
	{
		CubeCoordinates = float3(-1, -ScaledUVs.y, ScaledUVs.x);
	}
	else if (CubeFace == 2)
	{
		CubeCoordinates = float3(ScaledUVs.x, 1, ScaledUVs.y);
	}
	else if (CubeFace == 3)
	{
		CubeCoordinates = float3(ScaledUVs.x, -1, -ScaledUVs.y);
	}
	else if (CubeFace == 4)
	{
		CubeCoordinates = float3(ScaledUVs.x, -ScaledUVs.y, 1);
	}
	else
	{
		CubeCoordinates = float3(-ScaledUVs.x, -ScaledUVs.y, -1);
	}

	return CubeCoordinates;
}

TextureCube ShadowDepthCubeTexture;

void CopyCubeDepthPS(
	FCubeCopyGSToPS Input,
	out float OutDepth : SV_DEPTH,
	out float4 OutColor : SV_Target0
	)
{ 
	float2 ScaledUVs = Input.Vertex.UV * 2 - 1;
	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, Input.FaceIndex);

	OutColor = 0;
	OutDepth = TextureCubeSampleDepthLevel(ShadowDepthCubeTexture, ShadowDepthSampler, CubeCoordinates, 0);
}

float4 DepthOffsetScale;

void Scrolling2DDepthPS(
	FScreenVertexOutput Input,
	out float OutDepth : SV_DEPTH,
	out float4 OutColor : SV_Target0
)
{
	OutColor = 0;
	
	OutDepth = Texture2DSampleLevel(ShadowDepthTexture, ShadowDepthSampler, Input.UV, 0).x;

	OutDepth = OutDepth < 1.0f ? OutDepth * DepthOffsetScale.y + DepthOffsetScale.x : 1.0f;
}