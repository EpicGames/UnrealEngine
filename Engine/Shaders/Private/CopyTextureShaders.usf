// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CopyTextureShaders.usf:  
		Generic shader for copying 2D, 2D Array and 3D texture resources
=============================================================================*/

#include "Common.ush"

uint3 DstOffset;  // Front upper left texel in the destination texture to start writing data to
uint3 SrcOffset;  // Front upper left texel in the source texture to start reading data from
uint3 Dimensions; // The total x,y,z number of texels in the copy region

#if SRC_TYPE == 0

	Texture2D     <VALUE_TYPE> SrcResource;
	#define SRC_ADDR(p,o) ((p).xy + (o).xy)

#elif SRC_TYPE == 1

	Texture2DArray<VALUE_TYPE> SrcResource;
	#define SRC_ADDR(p,o) ((p).xyz + (o).xyz)

#elif SRC_TYPE == 2

	Texture3D     <VALUE_TYPE> SrcResource;
	#define SRC_ADDR(p,o) ((p).xyz + (o).xyz)

#endif

#if DST_TYPE == 0

	RWTexture2D     <VALUE_TYPE> DstResource;
	#define DST_ADDR(p,o) ((p).xy + (o).xy)

#elif DST_TYPE == 1

	RWTexture2DArray<VALUE_TYPE> DstResource;
	#define DST_ADDR(p,o) ((p).xyz + (o).xyz)

#elif DST_TYPE == 2

	RWTexture3D     <VALUE_TYPE> DstResource;
	#define DST_ADDR(p,o) ((p).xyz + (o).xyz)
	
#endif

#define COMP(a,b) (((a).x < (b).x) && ((a).y < (b).y) && ((a).z < (b).z))

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CopyTextureCS(uint3 Position : SV_DispatchThreadID)
{
	if (COMP(Position, Dimensions))
	{
		DstResource[DST_ADDR(Position, DstOffset)] = SrcResource[SRC_ADDR(Position, SrcOffset)];
	}
}
