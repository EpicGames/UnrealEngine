// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	CopyUIntBuffer.usf: Shader to copy one uint buffer into several overlapping targets.
	Each target is expected to be bigger than the previous one, so that copy resumes where it left.
	Ex : (ABCDEFGHIJK) -> (ABC--, ---DEFGHI-, ---------JK---)

	This shader is essentially used to manage growable buffers in cases the
	binding needs to be set before the final size required is known. 
	In this scenario, the smaller buffers are temporary and only the final (biggest) buffer
	becomes persistent, this is why it has apparently unused space at the beginning but
	it will be used the next frame to hold all of the data.
==============================================================================*/

/*------------------------------------------------------------------------------
	Compile time parameters:
		THREAD_COUNT - The number of threads to launch per workgroup.
------------------------------------------------------------------------------*/

#include "/Engine/Private/Common.ush"


// (StartingIndex, UsedIndexCount0, UsedIndexCount1, UsedIndexCount2)
uint4 CopyParams;


Buffer<uint> SourceData;

RWBuffer<uint> DestData0;
RWBuffer<uint> DestData1;
RWBuffer<uint> DestData2;

[numthreads(THREAD_COUNT,1,1)]
void MainCS(uint IndexOffset : SV_DispatchThreadID)
{
	const uint StartingIndex = CopyParams.x;
	const uint UsedIndexCount0 = CopyParams.y;
	const uint UsedIndexCount1 = CopyParams.z;
	const uint UsedIndexCount2 = CopyParams.w;

	uint Index = StartingIndex + IndexOffset;

	// The source buffer copy is splitted between the destination buffers.
	if (Index < UsedIndexCount0)
	{
		DestData0[Index] = SourceData[Index];
	}
	else if (Index < UsedIndexCount1)
	{
		DestData1[Index] = SourceData[Index];
	}
	else if (Index < UsedIndexCount2)
	{
		DestData2[Index] = SourceData[Index];
	}
}
 