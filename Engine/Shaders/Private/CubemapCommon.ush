// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CubemapCommon.usf: Cubemap shared functions and structures.
=============================================================================*/

// include common.usf to get PI

// 
TextureCube AmbientCubemap;
SamplerState AmbientCubemapSampler;

// .rgb:AmbientCubemapTint*AmbientCubemapIntensity, a:unused
half4 AmbientCubemapColor;
// .x:mul, .y:add for mip adjustement, z:DiffuseMip=MipCount-DiffuseConvolveMipLevel, w:MipCount
half4 AmbientCubemapMipAdjust;

// @param MipCount e.g. 10 for x 512x512
half ComputeCubemapMipFromRoughness( half Roughness, half MipCount )
{
	// Level starting from 1x1 mip
	half Level = 3 - 1.15 * log2( Roughness );
	return MipCount - 1 - Level;
}

// used for Translucency volume, can be optimized by precomputing the color on CPU side
float3 ComputeAmbientCubemapAvgColor()
{
	return TextureCubeSampleLevel(AmbientCubemap, AmbientCubemapSampler, float3(0, 0, 0), 20.0f).rgb * AmbientCubemapColor.rgb;
}