// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DBufferDecalShared.ush: Common definitions for DBuffer decals
=============================================================================*/

#pragma once

#ifndef DBUFFER_BASEPASS_RESOURCE
#define DBUFFER_BASEPASS_RESOURCE 1
#endif

#if DBUFFER_BASEPASS_RESOURCE

#if SHADING_PATH_MOBILE
#define DBufferBasePass MobileBasePass
#else
#define DBufferBasePass OpaqueBasePass
#endif

#define OpaqueBasePassDBufferATexture DBufferBasePass.DBufferATexture
#define OpaqueBasePassDBufferBTexture DBufferBasePass.DBufferBTexture
#define OpaqueBasePassDBufferCTexture DBufferBasePass.DBufferCTexture
#define OpaqueBasePassDBufferRenderMask DBufferBasePass.DBufferRenderMask

#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
#define OpaqueBasePassDBufferATextureArray DBufferBasePass.DBufferATextureArray
#define OpaqueBasePassDBufferBTextureArray DBufferBasePass.DBufferBTextureArray
#define OpaqueBasePassDBufferCTextureArray DBufferBasePass.DBufferCTextureArray
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define OpaqueBasePassDBufferATextureSampler DBufferBasePass.DBufferATextureSampler
#define OpaqueBasePassDBufferBTextureSampler DBufferBasePass.DBufferATextureSampler
#define OpaqueBasePassDBufferCTextureSampler DBufferBasePass.DBufferATextureSampler
#else
#define OpaqueBasePassDBufferATextureSampler DBufferBasePass.DBufferATextureSampler
#define OpaqueBasePassDBufferBTextureSampler DBufferBasePass.DBufferBTextureSampler
#define OpaqueBasePassDBufferCTextureSampler DBufferBasePass.DBufferCTextureSampler
#endif

#endif // DBUFFER_BASEPASS_RESOURCE

#if USE_DBUFFER && RAYHITGROUPSHADER
// This must be set prior to running the generated material code.
static float4 CurrentPayloadDBufferA = float4(0.0, 0.0, 0.0, 1.0);
static float4 CurrentPayloadDBufferB = float4(0.0, 0.0, 0.0, 1.0);
static float4 CurrentPayloadDBufferC = float4(0.0, 0.0, 0.0, 1.0);
#endif

// Returns a bitmask of DBuffer targets that contain valid data for this pixel.
// @param BufferUV - UV space in the DBuffer textures
uint GetDBufferTargetMask(uint2 PixelPos)
{
#if USE_DBUFFER && ENABLE_DBUFFER_TEXTURES && !RAYHITGROUPSHADER && !LUMEN_CARD_CAPTURE
	{
#	if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
		return DecodeRTWriteMask(PixelPos, OpaqueBasePassDBufferRenderMask, 3);
#	elif PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK
		uint Mask = OpaqueBasePassDBufferRenderMask.Load(uint3(PixelPos, 0));
		return Mask > 0 ? 0x07 : 0x00;
#	else
		return 0x07;
#	endif
	}
#elif USE_DBUFFER && RAYHITGROUPSHADER
	return 0x07;
#else
	return 0;
#endif
}

// all values that are output by the forward rendering pass
#if SUBSTRATE_ENABLED

void ConvertFromMetalness(
	in float3 BaseColor,
	in float Specular,
	in float Metallic,
	inout float3 DiffuseAlbedo,
	inout float3 F0)
{
	DiffuseAlbedo = lerp(BaseColor, 0, Metallic);
	F0			  = lerp(DielectricSpecularToF0(Specular), BaseColor, Metallic);
}

void ConvertToMetalness(
	in float3 DiffuseAlbedo,
	in float3 F0,
	inout float3 BaseColor,
	inout float Specular,
	inout float Metallic)
{
	Metallic  = F0RGBToMetallic(F0);
	Specular  = F0RGBToDielectricSpecular(F0);
	BaseColor = lerp(DiffuseAlbedo, F0, Metallic);
}

// @param BufferUV - UV space in the DBuffer textures
FSubstrateDBuffer SubstrateGetDBufferData(float2 BufferUV, uint RTMaskBit, uint EyeIndex)
{
	// SUBSTRATE_TODO: Add support MOBILE_MULTI_VIEW and INSTANCED_STEREO on mobile (using EyeIndex)

	// Setup default values, which mean that no decals are present in DBuffer
	float4 DBufferA = float4(0.0, 0.0, 0.0, 1.0);
	float4 DBufferB = float4(128.0f / 255.0f, 128.f / 255.5f, 128.f / 255.5f, 1.0);
	float4 DBufferC = float4(0.0, 0.0, 0.0, 1.0);

#if USE_DBUFFER && ENABLE_DBUFFER_TEXTURES && !RAYHITGROUPSHADER && !LUMEN_CARD_CAPTURE
	BRANCH
	if (RTMaskBit & 0x1) { DBufferA = Texture2DSampleLevel(OpaqueBasePassDBufferATexture, OpaqueBasePassDBufferATextureSampler, BufferUV, 0);	}
	BRANCH
	if (RTMaskBit & 0x2) { DBufferB = Texture2DSampleLevel(OpaqueBasePassDBufferBTexture, OpaqueBasePassDBufferBTextureSampler, BufferUV, 0);	}
	BRANCH
	if (RTMaskBit & 0x4) { DBufferC = Texture2DSampleLevel(OpaqueBasePassDBufferCTexture, OpaqueBasePassDBufferCTextureSampler, BufferUV, 0);	}
#elif USE_DBUFFER && RAYHITGROUPSHADER
	BRANCH
	if (RTMaskBit & 0x1) { DBufferA = CurrentPayloadDBufferA; }
	BRANCH // SubstrateUnpackDBuffer expects DBufferB to have a specific scale and bias
	if (RTMaskBit & 0x2) { DBufferB = float4(CurrentPayloadDBufferB.rgb * 0.5f + 128.0f / 255.0f, CurrentPayloadDBufferB.a); }
	BRANCH
	if (RTMaskBit & 0x4) { DBufferC = CurrentPayloadDBufferC; }
#endif
	return SubstrateUnpackDBuffer(DBufferA, DBufferB, DBufferC);
}

FSubstrateDBuffer SubstrateGetDBufferData(float2 BufferUV, uint RTMaskBit)
{
	return SubstrateGetDBufferData(BufferUV, RTMaskBit, 0 /*EyeIndex*/);
}

// Define how DBuffer is applied onto existing layers:
// * 0: Apply DBuffer as per-component parameter blending (legacy mode). Default. 
// * 1: Apply DBuffer as a coating layer (using parameter blending for efficiency purpose). Not used for now
#define SUBSTRATE_DBUFFER_LEGACY 0
#define SUBSTRATE_DBUFFER_COATING 1
#define SUBSTRATE_DBUFFER_BLENDING_MODE SUBSTRATE_DBUFFER_LEGACY

// Define which DBuffers are valid to read from, for this material
#define SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR_BIT_OFFSET 0
#define SUBSTRATE_DBUFFER_RESPONSE_NORMAL_BIT_OFFSET    1
#define SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS_BIT_OFFSET 2

#define SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR (MATERIALDECALRESPONSEMASK & (1<<SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR_BIT_OFFSET))
#define SUBSTRATE_DBUFFER_RESPONSE_NORMAL    (MATERIALDECALRESPONSEMASK & (1<<SUBSTRATE_DBUFFER_RESPONSE_NORMAL_BIT_OFFSET))
#define SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS (MATERIALDECALRESPONSEMASK & (1<<SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS_BIT_OFFSET))

#if MATERIAL_IS_SUBSTRATE
void ApplyDBufferData(in FSubstrateDBuffer SubstrateDBufferData, inout FSubstratePixelHeader SubstratePixelHeader, inout FSubstrateData SubstrateData)
{
	// We cannot early out due when coverage = 0 due to the possibility of using premultipled alpha.

	// For normal, apply decal data onto all basis, without consideration of top layers
	#if SUBSTRATE_DBUFFER_RESPONSE_NORMAL && SUBSTRATE_INLINE_SHADING
	if(any(SubstrateDBufferData.WorldNormal != 0.0) || SubstrateDBufferData.OneMinusCoverage_WorldNormal < 1.0)
	{
		UNROLL
		for (uint i = 0; i < SubstratePixelHeader.SharedLocalBases.Count; ++i)
		{
			// Note:
			// * Assume Normal is already been normalized
			// * After modification we normalize the normal to get smoother visual result (it helps to avoid having D_GGX explodes toward infinity)
			// 
			// SUBSTRATE_TODO: 
			// * If the normal is shared by different layers on which decals would be applied differently (e.g., one affected, not the other), we 
			//   would need to add a new normal entry and modify it. 
			// * If the basis has a tangent, the average operation should be changed into a basis sler/rotation
			SubstratePixelHeader.SharedLocalBases.Normals[i] = normalize(SubstratePixelHeader.SharedLocalBases.Normals[i] * SubstrateDBufferData.OneMinusCoverage_WorldNormal + SubstrateDBufferData.WorldNormal);
			//if (SubstrateGetSharedLocalBasisType(SubstratePixelHeader.SharedLocalBases.Types, i) == SUBSTRATE_BASIS_TYPE_TANGENT)
			//{
			//	SubstratePixelHeader.SharedLocalBases.Tangents[i] = normalize(SubstratePixelHeader.SharedLocalBases.Tangents[i]);
			//}
		}
	}
	#endif

	const bool bAffectDiffuse = any(SubstrateDBufferData.BaseColor > 0) || SubstrateDBufferData.OneMinusCoverage_BaseColor < 1.0;
	const bool bAffectF0Rough = any(SubstrateDBufferData.Metallic > 0) || any(SubstrateDBufferData.Specular > 0) || any(SubstrateDBufferData.Roughness > 0) || SubstrateDBufferData.OneMinusCoverage_Roughness < 1.0;
	if (!bAffectDiffuse && !bAffectF0Rough)
	{
		return;
	}

	// For now apply decal data onto all layers, without consideration of top layers
	Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
	{
	#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

		switch (BSDF_GETTYPE(BSDF))
		{
			case SUBSTRATE_BSDF_TYPE_SLAB:
			{	
				// Note:
				// * For Slab, since the DBuffer is stored with a Metalness parameterization for storage-space reason, 
				//   we were convert back&forth the Slab data between DiffuseAlbedo/F0 and Metalness. 
				// * However, this is a non identy operation even when non decal so material look could be changed.
				//   Furthermore this resulted in some transition edges being visible where the decal was being applied
				//=> We now do computation in Diffuse/F0 space. It is still not perfectly matching legacy 
				//   but at least the null operation is identity and no more transition edges are visible.
					
				// Apply DBuffer as per-component parameter blending (legacy mode)
				#if SUBSTRATE_DBUFFER_BLENDING_MODE == SUBSTRATE_DBUFFER_LEGACY
				{
					// The original metallic, before the decal touched the slab.
					// We use MIN on F0 to nor remove too much energy when metallic is lowered and have smoother metallic gradient with low F0.
					float SlabMetallic = F0ToMetallic(min(SLAB_F0(BSDF).r, min(SLAB_F0(BSDF).g, SLAB_F0(BSDF).b)));

				#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR || SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
					float3 DecalDiffuse;
					float3 DecalF0;
					ConvertFromMetalness(SubstrateDBufferData.BaseColor, SubstrateDBufferData.Specular, SubstrateDBufferData.Metallic, DecalDiffuse, DecalF0);

					if (bAffectDiffuse || bAffectF0Rough)
					{
						// Affect F0 if basecolor or metallic are changed.
						SLAB_F0(BSDF) = SLAB_F0(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor + DecalF0;
					}
				#endif

				#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
					if (bAffectDiffuse)
					{
						// Diffuse albedo is affected by throughpu and decal diffuse contribution
						SLAB_DIFFUSEALBEDO(BSDF) = SLAB_DIFFUSEALBEDO(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor	+ DecalDiffuse;

						// Fade out fuzz to avoid overbright decal.
						SLAB_FUZZ_AMOUNT(BSDF) *= SubstrateDBufferData.OneMinusCoverage_BaseColor;

						// Disable SSS when the coverage reach one to preserve the decal color. 
						// That operation also progressively makes non top layer disapear.
						// Since this is ran before the Substrate tree is processed, we must use TmpMFP and not SLAB_SSSMFP.
						BSDF.TmpMFP = BSDF.TmpMFP * SubstrateDBufferData.OneMinusCoverage_BaseColor;
						if (SubstrateDBufferData.OneMinusCoverage_BaseColor == 0.0f)
						{
							BSDF_SETSSSTYPE(BSDF, SSS_TYPE_NONE);
						}
					}
				#endif

				#if SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
					if (bAffectF0Rough)
					{
						// Now also operate a change to reflect the change of metallic
						float NewMetallic = SlabMetallic * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Metallic;
						
						if (NewMetallic >= SlabMetallic)
						{
							float TransferDelta		 = (NewMetallic - SlabMetallic);	// Positive or zero
							SLAB_F0(BSDF)			+= TransferDelta * SLAB_DIFFUSEALBEDO(BSDF);
							SLAB_DIFFUSEALBEDO(BSDF)-= TransferDelta * SLAB_DIFFUSEALBEDO(BSDF);
						}
						else
						{
							float TransferDelta =-(NewMetallic - SlabMetallic);			// Positive

							float CurrentSpecular		= F0ToDielectricSpecular(min(SLAB_F0(BSDF).r, min(SLAB_F0(BSDF).g, SLAB_F0(BSDF).b)));
							float F0ThatCannotTransfer	= DielectricSpecularToF0(CurrentSpecular);
							float3 F0ThatCanTransfer	= saturate(SLAB_F0(BSDF) - F0ThatCannotTransfer);

							SLAB_DIFFUSEALBEDO(BSDF)+= TransferDelta * F0ThatCanTransfer;
							SLAB_F0(BSDF)			-= TransferDelta * F0ThatCanTransfer;
						}
					
						SLAB_ROUGHNESS(BSDF)	= SLAB_ROUGHNESS(BSDF) * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Roughness;
					
						// Also fade out other specular features since we consider decal to have not specular features.
						if (BSDF_GETHASHAZINESS(BSDF))
						{
							FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
							Haziness.Weight *= SubstrateDBufferData.OneMinusCoverage_Roughness;
							SLAB_HAZINESS(BSDF) = PackHaziness(Haziness);
						}
					
						SLAB_ANISOTROPY(BSDF) *= SubstrateDBufferData.OneMinusCoverage_Roughness;
					}
				#endif
				}
				// Apply DBuffer as a coating layer (using parameter blending for efficiency purpose)
				#elif SUBSTRATE_DBUFFER_BLENDING_MODE == SUBSTRATE_DBUFFER_COATING
				{
					// Since all DBuffer parameters are not necessary initialized, take default value from the existing BDSF
					FSubstrateDBuffer LocalSubstrateDBufferData = SubstrateDBufferData;
					{
						float3 SlabBaseColor = 0;
						float  SlabMetallic = 0;
						float  SlabSpecular = 0;
						ConvertToMetalness(SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), SlabBaseColor, SlabSpecular, SlabMetallic);
					#if !SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
						LocalSubstrateDBufferData.BaseColor					= LocalSubstrateDBufferData.Coverage * SlabBaseColor;
						LocalSubstrateDBufferData.OneMinusCoverage_BaseColor	= LocalSubstrateDBufferData.OneMinusCoverage;
					#endif
					#if !SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
						LocalSubstrateDBufferData.Metallic						= LocalSubstrateDBufferData.Coverage * SlabMetallic;
						LocalSubstrateDBufferData.Specular						= LocalSubstrateDBufferData.Coverage * SlabSpecular;
						LocalSubstrateDBufferData.Roughness					= LocalSubstrateDBufferData.Coverage * SLAB_ROUGHNESS(BSDF);
						LocalSubstrateDBufferData.OneMinusCoverage_Roughness	= LocalSubstrateDBufferData.OneMinusCoverage;
					#endif
					}

					FSubstratePixelHeader SubstrateDecalHeader;
					FSubstrateData SubstrateDecalData;
					SubstrateDecalHeader.SubstrateConvertFromDBuffer(LocalSubstrateDBufferData, SubstrateDecalData);

					// Parameters blend attributes
					// SUBSTRATE_TODO BasisIndex is not defined... This is an experimental mode that will need revisiting when the time is right.
					FSubstrateData SubstrateDataForBSDF;
					SubstrateDataForBSDF.InlinedBSDF = BSDF;
					const float DummyNoV = 1.0f;
					BSDF = SubstrateVerticalLayeringParameterBlendingForDecal(SubstrateDecalData.Layers[l].BSDFs[i], SubstrateDataForBSDF, BasisIndex, DummyNoV, DummyNoV).InlinedBSDF; 
				}
				#endif // SUBSTRATE_DBUFFER_BLENDING_MODE
				break;
			}

			case SUBSTRATE_BSDF_TYPE_HAIR:
			{
			#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
				HAIR_BASECOLOR(BSDF) = HAIR_BASECOLOR(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor + SubstrateDBufferData.BaseColor;
			#endif

			#if SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
				HAIR_ROUGHNESS(BSDF) = HAIR_ROUGHNESS(BSDF) * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Roughness;
				HAIR_SPECULAR(BSDF)  = HAIR_SPECULAR(BSDF)  * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Specular;
			#endif
				break;
			}

			case SUBSTRATE_BSDF_TYPE_EYE:
			{
			#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
				EYE_DIFFUSEALBEDO(BSDF) = EYE_DIFFUSEALBEDO(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor + SubstrateDBufferData.BaseColor;
			#endif

			#if SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
				EYE_ROUGHNESS(BSDF) = EYE_ROUGHNESS(BSDF) * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Roughness;
			#endif
				break;
			}

			case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
			{
			#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
				SLW_BASECOLOR(BSDF) = SLW_BASECOLOR(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor + SubstrateDBufferData.BaseColor;
			#endif

			#if SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
				SLW_ROUGHNESS(BSDF) = SLW_ROUGHNESS(BSDF) * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Roughness;
				SLW_SPECULAR(BSDF)  = SLW_SPECULAR(BSDF)  * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Specular;
				SLW_METALLIC(BSDF)  = SLW_METALLIC(BSDF)  * SubstrateDBufferData.OneMinusCoverage_Roughness + SubstrateDBufferData.Metallic;
			#endif
				break;
			}

			case SUBSTRATE_BSDF_TYPE_TOON:
			{
			#if SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR
				TOON_BASECOLOR(BSDF) = TOON_BASECOLOR(BSDF) * SubstrateDBufferData.OneMinusCoverage_BaseColor + SubstrateDBufferData.BaseColor;
			#endif

			#if SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS
				const float DBufferAnisotropy = 0.0f;
				TOON_ROUGHNESS(BSDF)		= TOON_ROUGHNESS(BSDF) * SubstrateDBufferData.OneMinusCoverage_Roughness		+ SubstrateDBufferData.Roughness;
				TOON_SPECULAR(BSDF)			= TOON_SPECULAR(BSDF)  * SubstrateDBufferData.OneMinusCoverage_Roughness		+ SubstrateDBufferData.Specular;
			#endif
				break;
			}
		} // switch

	#undef BSDF
	}
}
#endif // MATERIAL_IS_SUBSTRATE

#endif // SUBSTRATE_ENABLED

struct FDBufferData
{
	// 0..1, premultiplied with ColorOpacity
	float3 PreMulColor;
	// 0:opaque ..1:see through
	float ColorOpacity;

	// -1..1, premultiplied with NormalOpacity
	float3 PreMulWorldNormal;
	// 0:opaque ..1:see through
	float NormalOpacity;

	// 0..1, premultiplied with RoughnessOpacity
	float PreMulRoughness;
	// 0..1, premultiplied with RoughnessOpacity
	float PreMulMetallic;
	// 0..1, premultiplied with RoughnessOpacity
	float PreMulSpecular;
	// 0:opaque ..1:see through
	float RoughnessOpacity;
};

/** Populates DBufferA, DBufferB, DBufferC as float4 and puts opacity in alpha for frame buffer blending */
// @param MultiOpacity .x: Color, .y:Normal, .z:Roughness/Metallic/Specular
void EncodeDBufferData(FGBufferData GBufferData, float3 MultiOpacity,
	out float4 DBufferA,
	out float4 DBufferB,
	out float4 DBufferC)
{
	// UNORM 4 channel
	DBufferA = float4(GBufferData.BaseColor, MultiOpacity.x);

	// UNORM 4 channel, 128/255 represents 0
	DBufferB = float4(GBufferData.WorldNormal * 0.5f + 128.0f/255.0f, MultiOpacity.y);

	// UNORM 4 channel
	DBufferC = float4(GBufferData.Metallic, GBufferData.Specular, GBufferData.Roughness, MultiOpacity.z);
}

/** Populates FDBufferData */
FDBufferData DecodeDBufferData(
	float4 DBufferA,
	float4 DBufferB,
	float4 DBufferC)
{
	FDBufferData ret;

	// UNORM 4 channel
	ret.PreMulColor = DBufferA.rgb;
	ret.ColorOpacity = DBufferA.a;

	// UNORM 4 channel, 128/255 represents 0
	ret.PreMulWorldNormal = DBufferB.rgb * 2 - (256.0 / 255.0);
	ret.NormalOpacity = DBufferB.a;

	// UNORM 4 channel
	ret.PreMulMetallic = DBufferC.r;
	ret.PreMulSpecular = DBufferC.g;
	ret.PreMulRoughness = DBufferC.b;
	ret.RoughnessOpacity = DBufferC.a;

	return ret;
}

// @param BufferUV - UV space in the DBuffer textures
FDBufferData GetDBufferData(float2 BufferUV, uint RTMaskBit, uint EyeIndex)
{
	// Setup default values, which mean that no decals are present in DBuffer
	float4 DBufferA = float4(0.0, 0.0, 0.0, 1.0);
	float4 DBufferB = float4(128.0f / 255.0f, 128.f / 255.5f, 128.f / 255.5f, 1.0);
	float4 DBufferC = float4(0.0, 0.0, 0.0, 1.0);

#if USE_DBUFFER && ENABLE_DBUFFER_TEXTURES && !RAYHITGROUPSHADER && !LUMEN_CARD_CAPTURE
	BRANCH
	if (RTMaskBit & 0x1)
	{
	#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
		DBufferA = Texture2DArraySampleLevel(OpaqueBasePassDBufferATextureArray, OpaqueBasePassDBufferATextureSampler, float3(BufferUV.xy, EyeIndex), 0);
	#else
		DBufferA = Texture2DSampleLevel(OpaqueBasePassDBufferATexture, OpaqueBasePassDBufferATextureSampler, BufferUV, 0);
	#endif
	}

	BRANCH
	if (RTMaskBit & 0x2)
	{
	#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
		DBufferB = Texture2DArraySampleLevel(OpaqueBasePassDBufferBTextureArray, OpaqueBasePassDBufferBTextureSampler, float3(BufferUV.xy, EyeIndex), 0);
	#else
		DBufferB = Texture2DSampleLevel(OpaqueBasePassDBufferBTexture, OpaqueBasePassDBufferBTextureSampler, BufferUV, 0);
	#endif
	}

	BRANCH
	if (RTMaskBit & 0x4)
	{
	#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
		DBufferC = Texture2DArraySampleLevel(OpaqueBasePassDBufferCTextureArray, OpaqueBasePassDBufferCTextureSampler, float3(BufferUV.xy, EyeIndex), 0);
	#else
		DBufferC = Texture2DSampleLevel(OpaqueBasePassDBufferCTexture, OpaqueBasePassDBufferCTextureSampler, BufferUV, 0);
	#endif
	}
#elif USE_DBUFFER && RAYHITGROUPSHADER
	BRANCH
	if (RTMaskBit & 0x1)
	{
		DBufferA = CurrentPayloadDBufferA;
	}

	BRANCH
	if (RTMaskBit & 0x2)
	{
		// DecodeDBufferData expects DBufferB to have a specific scale and bias
		DBufferB = float4(CurrentPayloadDBufferB.rgb * 0.5f + 128.0f / 255.0f, CurrentPayloadDBufferB.a);
	}

	BRANCH
	if (RTMaskBit & 0x4)
	{
		DBufferC = CurrentPayloadDBufferC;
	}
#endif

	return DecodeDBufferData(DBufferA, DBufferB, DBufferC);
}

FDBufferData GetDBufferData(float2 BufferUV, uint RTMaskBit)
{
	return GetDBufferData(BufferUV, RTMaskBit, 0);
}

/** Populates DBufferA, DBufferB, DBufferC as float4 and puts opacity in alpha for frame buffer blending */
void ApplyDBufferData(
	FDBufferData DBufferData, inout float3 WorldNormal, inout float3 SubsurfaceColor, inout float Roughness, 
	inout float3 BaseColor, inout float Metallic, inout float Specular )
{
#if (MATERIALDECALRESPONSEMASK & 0x1)
	BaseColor = BaseColor * DBufferData.ColorOpacity + DBufferData.PreMulColor;
	SubsurfaceColor *= DBufferData.ColorOpacity;
#endif

#if (MATERIALDECALRESPONSEMASK & 0x2)
	// We normalise the normal to get smoother visual result (it helps to avoid having D_GGX explodes toward infinity, and matches DecodeGBufferData)
	WorldNormal = normalize(WorldNormal * DBufferData.NormalOpacity + DBufferData.PreMulWorldNormal);
#endif

#if (MATERIALDECALRESPONSEMASK & 0x4)
	Roughness = Roughness * DBufferData.RoughnessOpacity + DBufferData.PreMulRoughness;
	Metallic = Metallic * DBufferData.RoughnessOpacity + DBufferData.PreMulMetallic;
	Specular = Specular * DBufferData.RoughnessOpacity + DBufferData.PreMulSpecular;
#endif
}
