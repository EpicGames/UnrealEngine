// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * DBufferNormalReprojection.usf: Utilities for reprojecting the previous frame's normal when read by the scene texture node in a DBuffer decal.
 */

#pragma once

#if IS_DBUFFER_DECAL && FEATURE_LEVEL >= FEATURE_LEVEL_SM5

/** Samples the screen-space velocity for the specified UV coordinates. 
 *  this one is slightly different from PostProcessVelocityLookup because it
 *  samples from SceneTexturesStruct_GBufferVelocityTextureSampler, and ignores
 *  the GBUFFER_HAS_VELOCITY #ifdef
 **/
float2 DeferredDecalPostProcessVelocityLookup(float Depth, float2 UV, float4 EncodedVelocity)
{
	float2 Velocity;
	if( EncodedVelocity.x > 0.0 )
	{
		Velocity = DecodeVelocityFromTexture(EncodedVelocity).xy;
	}
	else
	{
		float4 ThisClip = float4( UV, Depth, 1 );
		float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
		Velocity = UV - PrevScreen;
	}

	return Velocity;
}

// The functions TakeSmallerAbsDelta and ReconstructNormalFromDepthBuffer were copy/modified from PostProcessAmbientOcclusion.usf.
float TakeSmallerAbsDelta(float left, float mid, float right)
{
	float a = mid - left;
	float b = right - mid;

	return (abs(a) < abs(b)) ? a : b;
}

// could use ddx,ddy but that would have less quality and would nto work fo ComputeShaders
// @return not normalized normal in world space
float3 ReconstructNormalFromDepthBuffer(int2 BaseCoord)
{
	// could use a modified version of GatherSceneDepth later on
	float DeviceZ = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord + int2(0, 0),0)).x;
	float DeviceZLeft = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord + int2(-1, 0), 0)).x;
	float DeviceZTop = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord + int2(0, -1), 0)).x;
	float DeviceZRight = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord + int2(1, 0), 0)).x;
	float DeviceZBottom = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord + int2(0, 1), 0)).x;

	// Favor the surfae we are looking at. Simiar to: http://www.humus.name/index.php?page=3D&ID=84
	float DeviceZDdx = TakeSmallerAbsDelta(DeviceZLeft, DeviceZ, DeviceZRight);
	float DeviceZDdy = TakeSmallerAbsDelta(DeviceZTop, DeviceZ, DeviceZBottom);

	float2 SvPosition = float2(BaseCoord) + float2(.5f, .5f);

	// can be optimized, is not fully centered but that should not matter much
	float3 Mid =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(0, 0), DeviceZ, 1));
	float3 Right =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(1, 0), DeviceZ + DeviceZDdx, 1)) - Mid;
	float3 Down =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(0, 1), DeviceZ + DeviceZDdy, 1)) - Mid;

	return normalize(cross(Right, Down));
}

float4 GetDBufferReprojectedWorldNormal(float2 UV)
{
	int2 BaseCoord = (int2)trunc(UV * View.BufferSizeAndInvSize.xy);

	// When using Substrate with DBuffer pass, we can rely on TopLayerTexture to have been produced
	#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT == 1 && SUBSTRATE_USE_DBUFFER_PASS
	float3 N = SubstrateUnpackTopLayerData(SubstrateStruct.TopLayerTexture.Load(int3(BaseCoord, 0))).WorldNormal;
	#elif SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT == 0 && SUBSTRATE_USE_DBUFFER_PASS
	float3 N = GetGBufferDataUint(BaseCoord).WorldNormal;
	#else
	float DeviceZ = SceneTexturesStruct.SceneDepthTexture.Load(int3(BaseCoord,0)).x;

	// Velocity has to be the same size as the rest of the targets
	float4 EncodedVelocity = SceneTexturesStruct.GBufferVelocityTexture.Load(int3(BaseCoord,0));
	float3 FacetedNorm = ReconstructNormalFromDepthBuffer(BaseCoord);

	float3 N = FacetedNorm;

	if (DeferredDecal.NormalReprojectionEnabled)
	{
		// convert from [0,1] to [-1,1], and flip Y.
		float2 ViewportUV = (UV*2.0f - 1.0f) * float2(1.0f,-1.0f);

		float2 Velocity = DeferredDecalPostProcessVelocityLookup(DeviceZ, ViewportUV, EncodedVelocity);

		float4 ThisClip = float4(ViewportUV, DeviceZ, 1);
		float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
		float2 PrevScreen = PrevClip.xy / PrevClip.w;

		// Velocity is in clip space [-1,1] with flipped Y.
		// convert from [-1,1] to [0,1], and flip Y back.
		float2 ReprojUV = ((UV - Velocity*float2(1,-1)*.5f));

		bool bIsEdgeOcclusion = ReprojUV.x < 0 || ReprojUV.y < 0 || ReprojUV.x >= 1.0f || ReprojUV.y >= 1.0f;

		// Note: Doing a linear sample here, as it seems more correct since the normal buffer can blend between texels. Also,
		// we have the previous frame's jitter (DeferredDecal.NormalReprojectionJitter) available but it seems like we get the
		// best results by ignoring it. Also, it is possible that we have normal from a bad edge, but in that case it will
		// deviate too much from the faceted normal and get fixed a few lines later.
		float3 ReprojNormTex = Texture2DSampleLevel(DeferredDecal.PreviousFrameNormal, GlobalBilinearClampedSampler, ReprojUV, 0.0f).rgb;

		float3 ReprojNorm = normalize(ReprojNormTex * 2.0f - 1.0f);

		float DotFacet = saturate(dot(ReprojNorm,FacetedNorm));

		float Influence = saturate((DotFacet - DeferredDecal.NormalReprojectionThresholdLow) * DeferredDecal.NormalReprojectionThresholdScaleHelper);
		if (bIsEdgeOcclusion)
		{
			Influence = 0.0f;
		}
		N = normalize(lerp(FacetedNorm,ReprojNorm,Influence));
	}
	#endif

	return float4(N,1.0f);
}

#else // IS_DBUFFER_DECAL && FEATURE_LEVEL >= FEATURE_LEVEL_SM5

float4 GetDBufferReprojectedWorldNormal(float2 UV)
{
	// Stub function. Shouldn't ever be called.
	return float4(0, 0, 1, 1);
}

#endif // IS_DBUFFER_DECAL && FEATURE_LEVEL >= FEATURE_LEVEL_SM5