// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

// Mention that we are going to use a single slab
#define SUBSTRATE_CLAMPED_CLOSURE_COUNT 1
#define SUBSTRATE_INLINE_SHADING 1
#include "Substrate/Substrate.ush"

#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#include "Substrate/SubstrateExport.ush"

#include "DeferredShadingCommon.ush"
#include "VelocityCommon.ush"

#if SUBSTRATE_ENABLED
#include "SceneTexturesCommon.ush"
RWTexture2DArray<uint> MaterialTextureArrayUAV;
uint MaxBytesPerPixel;
uint bRoughDiffuse;

// Default functions for PackSubstrateOut to function properly
void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit(float3 V) {}
void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance(FSubstrateIntegrationSettings Settings, float3 V) {}
void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance() {}
void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit() {}
void FSubstratePixelHeader::PreUpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified(float3 V) {}
void FSubstratePixelHeader::UpdateAllBSDFsOperatorCoverageTransmittance_FullySimplified(FSubstrateIntegrationSettings Settings, float3 V) {}
void FSubstratePixelHeader::UpdateAllOperatorsCoverageTransmittance_FullySimplified() {}
void FSubstratePixelHeader::UpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified() {}

#endif // SUBSTRATE_ENABLED

int DebugProbesMode;

#define PROBE_MODE_SPECULAR	0
#define PROBE_MODE_DIFFUSE	1
#define PROBE_MODE_BOTH		2

// Must match DebugProbeRendering.usf
#define RENDER_DEPTHPREPASS	0
#define RENDER_BASEPASS		1
#define RENDER_VELOCITYPASS	2

// Base pass with Substrate cannot write to depth because it is going to execute in shader depth testing
#define DEPTH_OUTPUT_ENABLED (PERMUTATION_PASS == RENDER_VELOCITYPASS || PERMUTATION_PASS == RENDER_DEPTHPREPASS || (PERMUTATION_PASS == RENDER_BASEPASS && SUBSTRATE_ENABLED == 0))


void DebugSphereIntersection(
	float3 RayOrig, float3 RayDir, float3 SphereCenter, float SphereRadius, int NewProbeMode, in bool bNewProbeAttachedToCamera,
	inout int ProbeMode, inout float RayT, inout float3 WorlPosition, inout float3 Normal, inout bool bProbeAttachedToCamera)
{
	float2 RayTs = RayIntersectSphere(RayOrig, RayDir, float4(SphereCenter, SphereRadius));
	float NewRayT = max(0.0f, min(RayTs.x, RayTs.y));

	if (NewRayT > 0.0f && (NewRayT < RayT || RayT==-1.0f))
	{
		RayT = NewRayT;
		ProbeMode = NewProbeMode;
		WorlPosition = RayOrig + RayDir * RayT;
		Normal = normalize(WorlPosition - SphereCenter);
		bProbeAttachedToCamera = bNewProbeAttachedToCamera;

		if (ProbeMode == PROBE_MODE_BOTH)
		{
			bool bOn = Normal.z > 0.0f;
			bOn = Normal.y > 0.0f ? !bOn : bOn;
			bOn = Normal.x > 0.0f ? !bOn : bOn;
			ProbeMode = bOn ? PROBE_MODE_SPECULAR : PROBE_MODE_DIFFUSE;
		}
	}
}

#if PERMUTATION_PASS == RENDER_BASEPASS
void WriteOutMaterial(
	in uint2 PixelCoord,
	in float3 RayDir,
	in float PerObjectGBufferData,
	in float SelectiveOutputMask,
	in float3 BaseColor,
	in float  Metallic,
	in float  Specular,
	in float  Roughness,
	in float  IndirectIrradianceAndAO,
	in float3 Normal
#if SUBSTRATE_GBUFFER_FORMAT==1
	, inout float4 OutGBufferA
	, inout uint SubstrateMat0
	, inout uint SubstrateMat1
	, inout uint SubstrateMat2
	, inout SUBSTRATE_TOP_LAYER_TYPE SubstrateTopLayerData
	, inout uint OutGBufferE
#else
	, inout float4 OutGBufferA
	, inout float4 OutGBufferB
	, inout float4 OutGBufferC
	, inout float4 OutGBufferD
	, inout float4 OutGBufferE
#endif

)
{
#if SUBSTRATE_GBUFFER_FORMAT==1

#if SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT!=3
#error Please update debug probe MRT code.
#endif

	FSubstratePixelHeader SubstratePixelHeader = InitialiseSubstratePixelHeader();
	SubstratePixelHeader.SubstrateTree = GetInitialisedSubstrateTree();
	SubstratePixelHeader.IrradianceAO = InitIrradianceAndOcclusion();
	SubstratePixelHeader.IrradianceAO.MaterialAO = 1.0f - IndirectIrradianceAndAO;
	SubstratePixelHeader.SetCastContactShadow(0);

	uint SharedLocalBasisIndex = 0;
	const float3 NormalizedNormal = normalize(Normal);
	SubstratePixelHeader.SharedLocalBases.Normals[SharedLocalBasisIndex] = NormalizedNormal;
	SubstratePixelHeader.SharedLocalBases.Count = 1;

	FSubstratePixelFootprint PixelFootprint = (FSubstratePixelFootprint)0;
	FSubstrateData SubstrateSlabData = GetSubstrateSlabBSDF(
		/**FSubstratePixelFootprint PixelFootprint */PixelFootprint,
		/*float3 Normal							*/NormalizedNormal,
		/*float3 DiffuseAlbedo					*/ComputeDiffuseAlbedo(BaseColor, Metallic),
		/*float3 F0								*/ComputeF0(Specular, BaseColor, Metallic),
		/*float3 F90							*/1.0f,
		/*float Roughness						*/Roughness,
		/*float Anisotropy						*/0.0f,
		/*uint SSSProfileID						*/0,
		/*bool bSupportDefaultSSSProfile		*/false,
		/*float3 SSSMFP							*/0.0f,
		/*float SSSMFPScale						*/0.0f,
		/*float SSSPhaseAniso					*/0.0f,
		/*float SSSType							*/SSS_TYPE_NONE,
		/*float3 Emissive						*/0.0f,
		/*float SecondRoughness					*/0.0f,
		/*float SecondRoughnessWeight			*/0.0f,
		/*float SecondRoughnessAsSimpleClearCoat*/0.0f,
		/*float ClearCoatUseSecondNormal		*/0.0f,
		/*float3 ClearCoatBottomNormal			*/0.0f.xxx,
		/*float FuzzAmount						*/0.0f,
		/*float3 FuzzColor						*/0.0f,
		/*float FuzzRoughness					*/0.0f,
		/*float GlintValue						*/1.0f,
		/*float GlintUV							*/0.0f.xx,
		/*float SpecularProfile					*/0.0f,
		/*float Thickness						*/1.0f,
		/*bool bIsThinSurface					*/false,
		/*bool bIsAtTheBottomOfTopology			*/false,
		/*uint SharedLocalBasisIndex			*/SharedLocalBasisIndex,
		/*inout uint SharedLocalBasisTypes		*/SubstratePixelHeader.SharedLocalBases.Types);

	FSubstrateData SubstrateData = SubstratePixelHeader.SubstrateTree.PromoteParameterBlendedBSDFToOperator(SubstrateSlabData, /*int OperatorIndex*/0, /*int BSDFIndex*/0, /*int LayerDepth*/0, /*int bIsBottom*/1);
	SubstratePixelHeader.SubstrateTree.BSDFs[0].LuminanceWeightV = 1.0f;
	SubstratePixelHeader.SubstrateTree.BSDFs[0].TransmittanceAboveAlongN = 1.0f;
	SubstratePixelHeader.SubstrateTree.BSDFs[0].TopLayerDataWeight = 1.0f;

	FSubstrateSubsurfaceData SSSData = (FSubstrateSubsurfaceData)0;
	FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
	FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;

	float3 EmissiveLuminance = 0.0f;
	bool bSubstrateSubsurfaceEnable = false;
	const float Dither = 0.0f;
	const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, bRoughDiffuse, 0/*PeelLayersAboveDepth*/, false/*bRoughnessTracking*/);

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(ResolvedView.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
	FRWSubstrateMaterialContainer RWSubstrateMaterialContainer = InitialiseRWSubstrateMaterialContainer(MaterialTextureArrayUAV, 1/*QuadPixelWriteMask*/, PixelCoord);
	PackSubstrateOut(
		RWSubstrateMaterialContainer,
		MaterialTextureArrayUAV,
		Dither,
		Settings,
		SubstrateAddressing,
		SubstratePixelHeader, SubstrateData, -RayDir, NormalizedNormal, bSubstrateSubsurfaceEnable, EmissiveLuminance,
		SSSData, TopLayerData, OpaqueRoughRefractionData);

	// Write out MRTs. Note: this only works for deferred with basepassvelocity off
	OutGBufferA			= 0;
	SubstrateMat0			= RWSubstrateMaterialContainer.MaterialRenderTargets[0];
	SubstrateMat1			= RWSubstrateMaterialContainer.MaterialRenderTargets[1];
	SubstrateMat2			= RWSubstrateMaterialContainer.MaterialRenderTargets[2];
	SubstrateTopLayerData	= SubstratePackTopLayerData(TopLayerData);
	OutGBufferE			= 0;

#else // SUBSTRATE_ENABLED

	// We write all outputs, the rasterizer will discard what is not bound. That is fine since we do not care that much about performance for this debug view.

	OutGBufferA = float4(
		EncodeNormal(Normal),
		PerObjectGBufferData);

	OutGBufferB = float4(
		Metallic,
		Specular,
		Roughness,
		EncodeShadingModelIdAndSelectiveOutputMask(SHADINGMODELID_DEFAULT_LIT, SelectiveOutputMask));

	OutGBufferC = float4(BaseColor, IndirectIrradianceAndAO);
	OutGBufferD = float4(0.0f, 0.0f, 0.0f, 0.0f);	// custom data
	OutGBufferE = float4(1.0f, 1.0f, 1.0f, 1.0f);	// Pre computed shadow factor

#endif // SUBSTRATE_ENABLED
}
#endif // PERMUTATION_PASS == RENDER_BASEPASS



void MainPS(
	  in  float4 SvPosition		: SV_Position
	, out float4 OutColor		: SV_Target0
#if PERMUTATION_PASS == RENDER_BASEPASS
	#if SUBSTRATE_GBUFFER_FORMAT==1
	, out float4 OutGBufferA								: SV_Target2
	, out uint SubstrateMat0								: SV_Target3
	, out uint SubstrateMat1								: SV_Target4
	, out uint SubstrateMat2								: SV_Target5
	, out SUBSTRATE_TOP_LAYER_TYPE SubstrateTopLayerData	: SV_Target6
	, out uint OutGBufferE									: SV_Target7
	#else
	, out float4 OutGBufferA	: SV_Target1
	, out float4 OutGBufferB	: SV_Target2
	, out float4 OutGBufferC	: SV_Target3
	, out float4 OutGBufferD	: SV_Target4
	, out float4 OutGBufferE	: SV_Target5
	#endif
#endif
#if DEPTH_OUTPUT_ENABLED
	, out float  OutDepth		: SV_DEPTH
#endif
)
{
	OutColor = 0.f;
#if PERMUTATION_PASS == RENDER_BASEPASS
#if SUBSTRATE_GBUFFER_FORMAT==1
	OutGBufferA = 0;
	SubstrateMat0 = 0;
	SubstrateMat1 = 0;
	SubstrateMat2 = 0;
	SubstrateTopLayerData = 0;
	OutGBufferE = 0;
#else
	OutGBufferA = 0;
	OutGBufferB = 0;
	OutGBufferC = 0;
	OutGBufferD = 0;
	OutGBufferE = 0;
#endif
#endif

	ResolvedView = ResolveView();
	const uint2 PixelCoord = uint2(SvPosition.xy);

	float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;
	const float Depth = 1000000.0f;
	float4 TranslatedWorldPos = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, Depth), Depth, 1), View.ScreenToTranslatedWorld);

	const float3 RayDir = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPos.xyz);
	const float3 RayOrig = DFHackToFloat(GetWorldCameraPosFromView(ResolvedView, SvPosition.xy));

#if PERMUTATION_ILLUMINANCEMETER /////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	// In this case we want to show a small patch in front of the camera with pure white albedo.
	// That is a way to measure the illuminance at the camera position, on the hemisphere pointing towards the back of the camera.

	const float DeviceZ = 0.999999f;

	// IlluminanceMeter variables must match the one in PostProcessVisualizeHDR.usf
	const float  IlluminanceMeterSize = 20 * View.ResolutionFractionAndInv.x;
	const float  IlluminanceMeterHalfSize = IlluminanceMeterSize / 2;
	const float2 IlluminanceMeterPixelCenter = View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy * 0.5f - float2(0, 100) * View.ResolutionFractionAndInv.x;

	const float PixelMargin = 0.0f;
	if(all(PixelCoord > (IlluminanceMeterPixelCenter - IlluminanceMeterHalfSize - PixelMargin)) && all(PixelCoord < (IlluminanceMeterPixelCenter + IlluminanceMeterHalfSize + PixelMargin)))
	{
	#if PERMUTATION_PASS == RENDER_VELOCITYPASS

		float3 WorldPosition = RayOrig;
		float3 Velocity = 0.0f;
		float4 ScreenPos = mul(float4(WorldPosition.xyz + DFHackToFloat(ResolvedView.PreViewTranslation), 1), ResolvedView.TranslatedWorldToClip);
		float4 PrevScreenPos = mul(float4(WorldPosition.xyz + DFHackToFloat(ResolvedView.PrevPreViewTranslation), 1), ResolvedView.PrevTranslatedWorldToClip);
		Velocity = Calculate3DVelocity(ScreenPos, PrevScreenPos);
		OutColor = EncodeVelocityToTexture(Velocity);

	#elif PERMUTATION_PASS == RENDER_BASEPASS

		const float PerObjectGBufferData = 0.0f;
		const float SelectiveOutputMask = 0.0f;

		const float3 WorldNormal = -View.ViewForward;
		const float3 BaseColor = 1.0f;
		const float  Metallic = 0.0f;
		const float  Specular = 0.0f;
		const float  Roughness = 1.0f;
		const float  IndirectIrradianceAndAO = 0.0f;

	#if SUBSTRATE_ENABLED
		// EarlyDepthStencil does not work because we render a simple quad on screen with custom OutDepth.
		// We cannot rely on late depth test because pixel shader would have already operated the write to Substrate buffer.
		// So we must execute a manual depth test in shader (fine because basepass does not write depth)
		float2 UvBuffer = SvPosition.xy * View.BufferSizeAndInvSize.zw;	// Uv for depth buffer read (size can be larger than viewport)
		const float DepthBufferDeviceZ = LookupDeviceZ(UvBuffer);
		if (DepthBufferDeviceZ < (DeviceZ + 0.000001))
	#endif // SUBSTRATE_ENABLED
		{
			WriteOutMaterial(
				PixelCoord,
				RayDir,
				PerObjectGBufferData,
				SelectiveOutputMask,
				BaseColor,
				Metallic,
				Specular,
				Roughness,
				IndirectIrradianceAndAO,
				WorldNormal,
			#if SUBSTRATE_GBUFFER_FORMAT==1
				OutGBufferA,
				SubstrateMat0,
				SubstrateMat1,
				SubstrateMat2,
				SubstrateTopLayerData,
				OutGBufferE
			#else
				OutGBufferA,
				OutGBufferB,
				OutGBufferC,
				OutGBufferD,
				OutGBufferE
			#endif
			);
		}
	#endif

		OutColor = 0.0f;
	}
	else
	{
		discard;
	}



#else // PERMUTATION_ILLUMINANCEMETER /////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////



	int ProbeMode = false;
	bool bProbeAttachedToCamera = false;
	float RayT = -1.0f;
	float3 WorldPosition = 0.0f;
	float3 WorldNormal = 0.0f;


	// Render the camera probes (this could be done proceppdurally as flat sphere normal on screen rotated by inverse view matrix)
	if (DebugProbesMode == 1 || DebugProbesMode == 3)
	{
		{
			float3 SphereCenter = DFHackToFloat(ResolvedView.WorldCameraOrigin) + (ResolvedView.ViewForward * 1.0f + ResolvedView.ViewRight * 0.5f + ResolvedView.ViewUp * -0.2f) * METER_TO_CENTIMETER;
			float  SphereRadius = 0.2f * METER_TO_CENTIMETER;

			DebugSphereIntersection(
				RayOrig, RayDir, SphereCenter, SphereRadius, PROBE_MODE_SPECULAR, true,
				ProbeMode, RayT, WorldPosition, WorldNormal, bProbeAttachedToCamera);
		}
		{
			float3 SphereCenter = DFHackToFloat(ResolvedView.WorldCameraOrigin) + (ResolvedView.ViewForward * 1.0f + ResolvedView.ViewRight * -0.5f + ResolvedView.ViewUp * -0.2f) * METER_TO_CENTIMETER;
			float  SphereRadius = 0.2f * METER_TO_CENTIMETER;

			DebugSphereIntersection(
				RayOrig, RayDir, SphereCenter, SphereRadius, PROBE_MODE_DIFFUSE, true,
				ProbeMode, RayT, WorldPosition, WorldNormal, bProbeAttachedToCamera);
		}
	}


	// Render the world probes around the camera
	if (DebugProbesMode == 2 || DebugProbesMode == 3)
	{
		const float SphereRadius = 0.5f * METER_TO_CENTIMETER;

#if 1
		// Using Amanatides 3D-DDA to march virtual voxels around the camera and only evaluate intersection for sphere inside each voxel.
		const float3 VoxelSizeCm = float3(5.0f, 5.0f, 5.0f) * METER_TO_CENTIMETER;
		const float3 SnappedOrigin = (floor(DFHackToFloat(ResolvedView.WorldCameraOrigin) / VoxelSizeCm) + 0.5) * VoxelSizeCm
								   + float3(0.0f, 0.0f, 1.4 * SphereRadius);	// Small offset to have probes above the ground level 0

		const float3 BoxMin = SnappedOrigin - VoxelSizeCm * 4.0f;
		const float3 BoxMax = SnappedOrigin + VoxelSizeCm * 4.0f;

		float3 StartP = (RayOrig - BoxMin) / VoxelSizeCm;
		float3 P = StartP;

		// Amanatides 3D-DDA data preparation
		float3 stepSign = sign(RayDir);
		float3 tDelta = abs(1.0f / RayDir);
		float3 tMax = float3(0.0f, 0.0f, 0.0f);
		float3 refPoint = floor(P);
		tMax.x = stepSign.x > 0.0f ? refPoint.x + 1.0f - P.x : P.x - refPoint.x; // floor is more consistent than ceil
		tMax.y = stepSign.y > 0.0f ? refPoint.y + 1.0f - P.y : P.y - refPoint.y;
		tMax.z = stepSign.z > 0.0f ? refPoint.z + 1.0f - P.z : P.z - refPoint.z;
		tMax.x *= tDelta.x;
		tMax.y *= tDelta.y;
		tMax.z *= tDelta.z;

		LOOP
		while (dot(StartP - P, StartP - P) <= 5.1f * 5.1f)	// stop after a distance of 5 voxels is reached 
		{
			const float3 SphereCenter = BoxMin + (floor(P) + 0.5) * VoxelSizeCm;

			DebugSphereIntersection(
				RayOrig, RayDir, SphereCenter, SphereRadius, PROBE_MODE_BOTH, false,
				ProbeMode, RayT, WorldPosition, WorldNormal, bProbeAttachedToCamera);

			// Amanatides 3D-DDA 
			if (tMax.x < tMax.y)
			{
				if (tMax.x < tMax.z)
				{
					P.x += stepSign.x;
					tMax.x += tDelta.x;
				}
				else
				{
					P.z += stepSign.z;
					tMax.z += tDelta.z;
				}
			}
			else
			{
				if (tMax.y < tMax.z)
				{
					P.y += stepSign.y;
					tMax.y += tDelta.y;
				}
				else
				{
					P.z += stepSign.z;
					tMax.z += tDelta.z;
				}
			}
		}

#else
		// Simple but slow volume around the camera
		const float3 Steps = float3(5.0f, 5.0f, 2.5f);
		const float3 SnappedOrigin = floor(DFHackToFloat(ResolvedView.WorldCameraOrigin) / (Steps * METER_TO_CENTIMETER) + 0.5) * Steps * METER_TO_CENTIMETER
								   + float3(0.0f, 0.0f, 1.4 * SphereRadius);	// Small offset to have probes above the ground level 0
		LOOP
		for (float x = -15.0f * METER_TO_CENTIMETER; x <= 15.0f * METER_TO_CENTIMETER; x += Steps.x * METER_TO_CENTIMETER)
		{
			LOOP
			for (float y = -15.0f * METER_TO_CENTIMETER; y <= 15.0f * METER_TO_CENTIMETER; y += Steps.y * METER_TO_CENTIMETER)
			{
				LOOP
				for (float z = -5.0f * METER_TO_CENTIMETER; z <= 5.0f * METER_TO_CENTIMETER; z += Steps.z * METER_TO_CENTIMETER)
				{
					float3 SphereCenter = SnappedOrigin + float3(x, y, z);

					DebugSphereIntersection(
						RayOrig, RayDir, SphereCenter, SphereRadius, PROBE_MODE_BOTH, false,
						ProbeMode, RayT, WorldPosition, WorldNormal, bProbeAttachedToCamera);
				}
			}
		}
#endif
	}


	float4 ClipPos = mul(float4(WorldPosition, 1.0f), DFHackToFloat(ResolvedView.WorldToClip));
	ClipPos /= ClipPos.w;
	const float DeviceZ = ClipPos.z;
#if DEPTH_OUTPUT_ENABLED
	OutDepth = INVARIANT(DeviceZ);
#endif

	if (RayT <= 0.0f)
	{
		discard;
	}


#if PERMUTATION_PASS == RENDER_VELOCITYPASS

	float3 Velocity = 0.0f;

	if (!bProbeAttachedToCamera)
	{
		float4 ScreenPos	= mul(float4(WorldPosition.xyz + DFHackToFloat(ResolvedView.PreViewTranslation), 1),	ResolvedView.TranslatedWorldToClip);
		float4 PrevScreenPos= mul(float4(WorldPosition.xyz + DFHackToFloat(ResolvedView.PrevPreViewTranslation), 1),ResolvedView.PrevTranslatedWorldToClip);
		Velocity = Calculate3DVelocity(ScreenPos, PrevScreenPos);
	}

	OutColor = EncodeVelocityToTexture(Velocity);

#elif PERMUTATION_PASS == RENDER_BASEPASS

	const float PerObjectGBufferData = 0.0f;
	const float SelectiveOutputMask = 0.0f;

	const float3 BaseColor = 1.0f;
	const float  Metallic = ProbeMode == PROBE_MODE_DIFFUSE ? 0.0f : 1.0f;
	const float  Specular = ProbeMode == PROBE_MODE_DIFFUSE ? 0.0f : 1.0f;
	const float  Roughness = 0.01f;
	const float  IndirectIrradianceAndAO = 0.0f;

#if SUBSTRATE_ENABLED
	// EarlyDepthStencil does not work because we render a simple quad on screen with custom OutDepth.
	// We cannot rely on late depth test because pixel shader would have already operated the write to Substrate buffer.
	// So we must execute a manual depth test in shader (fine because basepass does not write depth)
	float2 UvBuffer = SvPosition.xy * View.BufferSizeAndInvSize.zw;	// Uv for depth buffer read (size can be larger than viewport)
	const float DepthBufferDeviceZ = LookupDeviceZ(UvBuffer);
	if (DepthBufferDeviceZ < (DeviceZ + 0.000001))
#endif // SUBSTRATE_ENABLED
	{
		WriteOutMaterial(
			PixelCoord,
			RayDir,
			PerObjectGBufferData,
			SelectiveOutputMask,
			BaseColor,
			Metallic,
			Specular,
			Roughness,
			IndirectIrradianceAndAO,
			WorldNormal,
		#if SUBSTRATE_GBUFFER_FORMAT==1
			OutGBufferA,
			SubstrateMat0,
			SubstrateMat1,
			SubstrateMat2,
			SubstrateTopLayerData,
			OutGBufferE
		#else
			OutGBufferA,
			OutGBufferB,
			OutGBufferC,
			OutGBufferD,
			OutGBufferE
		#endif
		);
	}
#if SUBSTRATE_ENABLED
	else
	{
		discard;
	}
#endif

	OutColor = 0.0f;

#endif // PERMUTATION_PASS

#endif // PERMUTATION_ILLUMINANCEMETER /////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if DEPTH_OUTPUT_ENABLED
	OutDepth = DeviceZ;
#endif

}