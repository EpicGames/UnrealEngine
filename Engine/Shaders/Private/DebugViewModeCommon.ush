// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugViewModeCommon.ush: Debug pixel shader interpolants.
=============================================================================*/

// This define the interpolant needed by the pixel shader used for debug viewmode.
// Those use a global shader that must have exactly the same inputs as the VS outputs.
// This is also compatible with shaders that modify the mesh position.
// Which is better in the end than simply defining a material override.

struct FDebugVSToPS
{
	// TexCoords must first in order to be compatible with viewmode requiring using only SV_Position
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
	float4 VertexColor : TEXCOORD0;
	float4 TexCoord01 : TEXCOORD1;
	float4 TexCoord23 : TEXCOORD2;
	float3 TangentToWorld0 : TEXCOORD3;
	float3 TangentToWorld1 : TEXCOORD4;
	float3 TangentToWorld2 : TEXCOORD5;
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
	nointerpolation uint InstanceId : INSTANCE_ID;
	FStereoVSOutput StereoOutput;
};

struct FDebugPSIn
{
	float4 SvPosition : SV_POSITION;
	float4 VertexColor : TEXCOORD0;
	float4 TexCoord01 : TEXCOORD1;
	float4 TexCoord23 : TEXCOORD2;
	float3 TangentToWorld0 : TEXCOORD3;
	float3 TangentToWorld1 : TEXCOORD4;
	float3 TangentToWorld2 : TEXCOORD5;
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
	nointerpolation uint InstanceId : INSTANCE_ID;
	FStereoPSInput StereoInput;
#if (OUTPUT_QUAD_OVERDRAW)
	uint SvPrimitiveID : SV_PrimitiveID;
#endif
};

// This one is only compatible if no previous stage outputed SV_PrimitiveID (i.e. geometry shader)
struct FDebugPSInLean
{
	float4 SvPosition : SV_POSITION;
	uint PrimitiveID : SV_PrimitiveID;
};

// Reroute DebugViewModeStruct uniform buffer references
#if SHADING_PATH_MOBILE
#define DebugViewModeStruct MobileBasePass.DebugViewMode
#define MobileSceneTextures MobileBasePass.SceneTextures 
#else
#define DebugViewModeStruct DebugViewModePass.DebugViewMode
#define SceneTexturesStruct DebugViewModePass.SceneTextures 
#endif

#if DEBUG_MATERIAL_PARAMETERS
/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FDebugPSIn DebugInputs, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

	Result.VertexColor = DebugInputs.VertexColor;

	Result.TangentToWorld = half3x3(half3(DebugInputs.TangentToWorld0), half3(DebugInputs.TangentToWorld1), half3(DebugInputs.TangentToWorld2));

#if NUM_MATERIAL_TEXCOORDS > 0
	Result.TexCoords[0].xy = DebugInputs.TexCoord01.xy;
#endif
#if NUM_MATERIAL_TEXCOORDS > 1
	Result.TexCoords[1].xy = DebugInputs.TexCoord01.zw;
#endif
#if NUM_MATERIAL_TEXCOORDS > 2
	Result.TexCoords[2].xy = DebugInputs.TexCoord23.xy;
#endif
#if NUM_MATERIAL_TEXCOORDS > 3
	Result.TexCoords[3].xy = DebugInputs.TexCoord23.zw;
#endif
#if NUM_MATERIAL_TEXCOORDS > 4
	for(int CoordinateIndex = 4; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = DebugInputs.TexCoord01.xy;
	}
#endif

	Result.TwoSidedSign = 1;
	return Result;
}
#endif