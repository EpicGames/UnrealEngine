// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugViewModePixelShader.usf: Ubershader for debug pixel modes.
=============================================================================*/

#define TEX_COORD_SCALE_ANALYSIS 1
#define DEBUG_MATERIAL_PARAMETERS 1

// Don't allow scene texture as we use ESceneRenderTargetsMode::DontSet in FMaterialTexCoordScalePS::SetParameters
// to avoid crashes in commandlet mode.
#undef SCENE_TEXTURES_DISABLED
#define SCENE_TEXTURES_DISABLED 1

#undef EYE_ADAPTATION_DISABLED
#define EYE_ADAPTATION_DISABLED 1

#include "Common.ush"
#include "SHCommon.ush"
#if OUTPUT_QUAD_OVERDRAW
#include "QuadOverdraw.ush"
#endif
#include "VirtualTextureCommon.ush"

#define UNDEF_ACCURACY 1024

// If -1, output everything to the RW texture.
float4 OneOverCPUTexCoordScales[MAX_NUM_TEXTURE_REGISTER / 4];
int4 TexCoordIndices[MAX_NUM_TEXTURE_REGISTER / 4];
float4 CPUTexelFactor; // In world space units.
float4 NormalizedComplexity;
int2 AnalysisParams; // (TextureAnalysisIndex, bOutputScales) or // (TextureAnalysisIndex, TextureResolution)
float PrimitiveAlpha;
int PrimitiveBlendMode;
int TexCoordAnalysisIndex;
float CPULogDistance;
uint bShowQuadOverdraw;
uint bOutputQuadOverdraw;
int LODIndex;
float3 SkinCacheDebugColor;
int VisualizeMode;

// Analysis index will be in [0, 31] if analysing a single texture, otherwise -1
#define TextureAnalysisIndex (AnalysisParams.x)
#define bOutputScales (AnalysisParams.y)
// Texture resolution if we're analysing a single texture
#define TextureResolution (AnalysisParams.y)

// Updated MAX_NUM_TEXTURE_REGISTER to 256 

float GetComponent(float4 V, int Index)
{
	FLATTEN
		if (Index == 0) return V.x;
	FLATTEN
		if (Index == 1) return V.y;
	FLATTEN
		if (Index == 2) return V.z;
	return V.w;
}

struct FTexCoordScalesParams
{
	// Global var used to reduce the number of param passed to the shaders.
	int2 PixelPosition;

	// ddU, ddV for each coord index (up to 4 currently)
	float4 OneOverDDU;
	float4 OneOverDDV;

	// Used when inspecting single elements.
	float MinScale;
	float MaxScale;

	float TexSample;
	float TexSampleAverage;

	// When computing the texture scales, this holds 4 scales for consecutive texture register indices.
	float4 ScalesPerIndex;

	float RequiredResolution;
};

MaterialFloat StoreTexCoordScale(in out FTexCoordScalesParams Params, float2 UV, int TextureReferenceIndex)
{
	// Take the minimum scale since high scale requires less resolution.
	float GPUScaleX = length(ddx(UV));
	float GPUScaleY = length(ddy(UV));

	if (TextureReferenceIndex >= 0 && TextureReferenceIndex < MAX_NUM_TEXTURE_REGISTER)
	{
		float OneOverCPUScale = OneOverCPUTexCoordScales[TextureReferenceIndex / 4][TextureReferenceIndex % 4];

		int TexCoordIndex = TexCoordIndices[TextureReferenceIndex / 4][TextureReferenceIndex % 4];

		float GPUScale = min(GPUScaleX * GetComponent(Params.OneOverDDU, TexCoordIndex), GPUScaleY * GetComponent(Params.OneOverDDV, TexCoordIndex));

		// Code path when searching the worst scales. MinScale and MaxScale are only used when rendering for accuracies (!bOutputScales)
		const bool bUpdateMinMax = (OneOverCPUScale > 0 && (TextureAnalysisIndex == -1 || TextureAnalysisIndex == TextureReferenceIndex));
		Params.MinScale = bUpdateMinMax ? min(Params.MinScale, GPUScale * OneOverCPUScale) : Params.MinScale;
		Params.MaxScale = bUpdateMinMax ? max(Params.MaxScale, GPUScale * OneOverCPUScale) : Params.MaxScale;

		// Code path when rendering the material in a render target. Each TILE_RESOLUTION X TILE_RESOLUTION maps to different texture.
		const bool bUpdateScale = (bOutputScales && Params.PixelPosition.x / TILE_RESOLUTION == TextureReferenceIndex / 4);
		Params.ScalesPerIndex[TextureReferenceIndex % 4] = bUpdateScale ? min(Params.ScalesPerIndex[TextureReferenceIndex % 4], GPUScale) : Params.ScalesPerIndex[TextureReferenceIndex % 4];
	}

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	float2 CoordDDX = ddx_fine(UV);
	float2 CoordDDY = ddy_fine(UV);
#else
	float2 CoordDDX = ddx(UV);
	float2 CoordDDY = ddy(UV);
#endif 

	if (TextureAnalysisIndex == TextureReferenceIndex)
	{
		float MinDelta = min(length(CoordDDX), length(CoordDDY));
		float RequiredResolution = (1 / max(MinDelta, 0.0000000001f));
		Params.RequiredResolution = max(Params.RequiredResolution, RequiredResolution);
	}

	return 1.f;
}

MaterialFloat StoreTexSample(in out FTexCoordScalesParams Params, float4 C, int TextureReferenceIndex)
{
	// Alpha?
	if (TextureAnalysisIndex == TextureReferenceIndex)
	{
		Params.TexSample = TextureAnalysisIndex == TextureReferenceIndex ? lerp(.4f, 1.f, saturate(Luminance(C.rgb))) : Params.TexSample;
	}
	Params.TexSampleAverage = TextureAnalysisIndex == TextureReferenceIndex ? lerp(.4f, 1.f, saturate((C.r + C.g + C.b) / 3)) : Params.TexSample;

	return 1.f;
}

#define SUBSTRATE_INLINE_SHADING 1
#include "Substrate/Substrate.ush"
#if SUBSTRATE_MATERIAL_EXPORT_EXECUTED || SUBSTRATE_GBUFFER_FORMAT==0
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#define SUBSTRATE_MATERIAL_EXPORT_REQUESTED 1
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

#include "/Engine/Generated/Material.ush"
#include "BasePassCommonFunctions.ush"
#include "DebugViewModeCommon.ush"

float3 AccuracyColorLookup(float Accuracy)
{
	Accuracy = clamp(Accuracy, -1.99, 1.99);
	int ColorIndex = floor(Accuracy) + 2;
	float3 Color0 = DebugViewModeStruct.AccuracyColors[ColorIndex].rgb;
	float3 Color1 = DebugViewModeStruct.AccuracyColors[ColorIndex + 1].rgb;
	float ColorLerp = frac(Accuracy);
	return lerp(Color0, Color1, ColorLerp);
}

float GetCPUTexelFactor(int CoordIndex)
{
	FLATTEN
		if (CoordIndex == 0) return CPUTexelFactor.x;
	FLATTEN
		if (CoordIndex == 1) return CPUTexelFactor.y;
	FLATTEN
		if (CoordIndex == 2) return CPUTexelFactor.z;
	return CPUTexelFactor.w;
}

float2 GetTexCoord(in FDebugPSIn Inputs, int CoordIndex)
{
	FLATTEN
		if (CoordIndex == 0) return Inputs.TexCoord01.xy;
	FLATTEN
		if (CoordIndex == 1) return Inputs.TexCoord01.zw;
	FLATTEN
		if (CoordIndex == 2) return Inputs.TexCoord23.xy;
	return Inputs.TexCoord23.zw;
}

float GetTexCoordSizeAccuracy(in FDebugPSIn Inputs, int CoordIndex, float CPUSize)
{
	float3 WorldPosition = SvPositionToResolvedTranslatedWorld(Inputs.SvPosition);

	float2 TexCoord = GetTexCoord(Inputs, CoordIndex);

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	float2 CoordDDX = ddx_fine(TexCoord);
	float2 CoordDDY = ddy_fine(TexCoord);
	float3 WorldPosDDX = ddx_fine(WorldPosition);
	float3 WorldPosDDY = ddy_fine(WorldPosition);
#else
	float2 CoordDDX = ddx(TexCoord);
	float2 CoordDDY = ddy(TexCoord);
	float3 WorldPosDDX = ddx(WorldPosition);
	float3 WorldPosDDY = ddy(WorldPosition);
#endif 

	float UVAera = abs(CoordDDX.x * CoordDDY.y - CoordDDX.y * CoordDDY.x);
	float WorldAera = length(cross(WorldPosDDX, WorldPosDDY));

	float GPUTexelFactor = sqrt(WorldAera / max(UVAera, 0.0000000001f));

	return clamp(log2(CPUSize) - log2(GPUTexelFactor), -1.99, 1.99);
}

float4 VisualizeMaterialTexCoordScales(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	// This default value will make it dark grey when nothing updates the texSample.
	float3 Result = float3(UNDEFINED_ACCURACY, UNDEFINED_ACCURACY, UNDEFINED_ACCURACY);

	FTexCoordScalesParams Params;

	// Set global var used for outputting data
	Params.PixelPosition = DebugInputs.SvPosition.xy;
	Params.OneOverDDU = 1 / float4(length(ddx(DebugInputs.TexCoord01.xy)), length(ddx(DebugInputs.TexCoord01.zw)), length(ddx(DebugInputs.TexCoord23.xy)), length(ddx(DebugInputs.TexCoord23.zw)));
	Params.OneOverDDV = 1 / float4(length(ddy(DebugInputs.TexCoord01.xy)), length(ddy(DebugInputs.TexCoord01.zw)), length(ddy(DebugInputs.TexCoord23.xy)), length(ddy(DebugInputs.TexCoord23.zw)));

	Params.TexSample = 1.f;
	Params.TexSampleAverage = 1.f;
	Params.MinScale = INITIAL_GPU_SCALE;
	Params.MaxScale = 0;
	Params.ScalesPerIndex = INITIAL_GPU_SCALE;
	Params.RequiredResolution = 0;

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(DebugInputs, DebugInputs.SvPosition);

	// This is used to patch invalid parameters when computing the scales with FMeshMaterialRenderItem::EnqueueMaterialRender
	FLATTEN
		if (bOutputScales)
		{
#if NUM_MATERIAL_TEXCOORDS > 0
			// Set all texcoord but one to 0. This allows the analysis to figure out which texcoord is driving the lookup.
			int CurrentCoordIndex = (Params.PixelPosition.y / TILE_RESOLUTION) % MAX_NUM_TEX_COORD;

			UNROLL
				for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; ++CoordinateIndex)
				{
					// The quad material used only has a valid TexCoord for UV 0
					MaterialParameters.TexCoords[CoordinateIndex] = CoordinateIndex != CurrentCoordIndex ? 0 : DebugInputs.TexCoord01.xy;
				}

			// The scale must also be computed to match the actual coord inspected. That would be required when rendering the mesh in world space
			// Although not required here as using the material quad only use texcoord0.
			// Params.OneOverDDU = GetComponent(Params.OneOverDDU, CurrentCoordIndex).xxxx;
			// Params.OneOverDDV = GetComponent(Params.OneOverDDV, CurrentCoordIndex).xxxx;
#endif
		}

	half3 BaseColor;
	MaterialParameters.TexCoordScalesParams = Params;
	{
		FPixelMaterialInputs PixelMaterialInputs;
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, DebugInputs.SvPosition, bIsFrontFace);

		// Sample material properties. The results are not used, but the calls to StoreTexCoordScale will still be made.
		BaseColor = GetMaterialBaseColorRaw(PixelMaterialInputs);
		half  Metallic = GetMaterialMetallicRaw(PixelMaterialInputs);
		half  Specular = GetMaterialSpecularRaw(PixelMaterialInputs);
		float Roughness = GetMaterialRoughnessRaw(PixelMaterialInputs);
		half3 Normal = GetMaterialNormalRaw(PixelMaterialInputs);
		half3 Emissive = GetMaterialEmissiveRaw(PixelMaterialInputs);
		half Opacity = GetMaterialOpacityRaw(PixelMaterialInputs);
#if MATERIALBLENDING_MASKED
		half Mask = GetMaterialMask(PixelMaterialInputs);
		if (!bOutputScales)
		{
			clip(GetMaterialMask(PixelMaterialInputs));
		}
#endif
		half4 SSData = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs);
		float Custom0 = GetMaterialCustomData0(PixelMaterialInputs);
		float Custom1 = GetMaterialCustomData1(PixelMaterialInputs);
		float MaterialAO = GetMaterialAmbientOcclusionRaw(PixelMaterialInputs);
		float PixelDepthOffset = GetMaterialPixelDepthOffset(PixelMaterialInputs);
#if CLEAR_COAT_BOTTOM_NORMAL && NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
		float3 BottomNormal = ClearCoatBottomNormal0(MaterialParameters);
#endif
	}
	Params = MaterialParameters.TexCoordScalesParams;

	Result *= saturate(Luminance(BaseColor));

	float4 OutColor;
	if (bOutputScales)
	{
		// Output the scales
		OutColor = Params.ScalesPerIndex;
	}
	else
	{
		float PixelScale = ((Params.PixelPosition.x & 0x08) == (Params.PixelPosition.y & 0x08) || TextureAnalysisIndex != -1) ? Params.MinScale : Params.MaxScale;

		// Output accuracy. If the GPU Scale is INITIAL_GPU_SCALE, then the texcoord was not updated.
		if (Params.MinScale != INITIAL_GPU_SCALE)
		{
			Params.TexSample = TextureAnalysisIndex == -1 ? lerp(.4f, 1.f, saturate(Luminance(BaseColor))) : Params.TexSample;

			float Accuracy = clamp(log2(PixelScale), -1.99, 1.99);
			int ColorIndex = floor(Accuracy) + 2;
			Result = Params.TexSample * lerp(DebugViewModeStruct.AccuracyColors[ColorIndex].rgb, DebugViewModeStruct.AccuracyColors[ColorIndex + 1].rgb, frac(Accuracy));
		}

		OutColor = float4(Result, PrimitiveAlpha);
	}
	return OutColor;
}

float4 VisualizeMeshTexCoordSizeAccuracy(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	float3 Result = float3(UNDEFINED_ACCURACY, UNDEFINED_ACCURACY, UNDEFINED_ACCURACY);

	if (TexCoordAnalysisIndex >= 0)
	{
		float CPUSize = GetCPUTexelFactor(TexCoordAnalysisIndex);
		if (CPUSize > 0)
		{
			float Accuracy = GetTexCoordSizeAccuracy(DebugInputs, TexCoordAnalysisIndex, CPUSize);
			int ColorIndex = floor(Accuracy) + 2;
			Result = lerp(DebugViewModeStruct.AccuracyColors[ColorIndex].rgb, DebugViewModeStruct.AccuracyColors[ColorIndex + 1].rgb, frac(Accuracy));
		}
	}
	else
	{
		float MinAccuracy = UNDEF_ACCURACY;
		float MaxAccuracy = -UNDEF_ACCURACY;

		[unroll]
		for (int CoordIndex = 0; CoordIndex < 4; ++CoordIndex)
		{
			float CPUSize = GetCPUTexelFactor(CoordIndex);
			if (CPUSize > 0)
			{
				float Accuracy = GetTexCoordSizeAccuracy(DebugInputs, CoordIndex, CPUSize);
				MinAccuracy = min(Accuracy, MinAccuracy);
				MaxAccuracy = max(Accuracy, MaxAccuracy);
			}
		}

		int2 PixelPosition = DebugInputs.SvPosition.xy;
		float Accuracy = (PixelPosition.x & 0x08) == (PixelPosition.y & 0x08) ? MinAccuracy : MaxAccuracy;

		if (abs(Accuracy) != UNDEF_ACCURACY)
		{
			int ColorIndex = floor(Accuracy) + 2;
			Result = lerp(DebugViewModeStruct.AccuracyColors[ColorIndex].rgb, DebugViewModeStruct.AccuracyColors[ColorIndex + 1].rgb, frac(Accuracy));
		}
	}

	return float4(Result, PrimitiveAlpha);
}

float4 VisualizePrimitiveDistanceAccuracy(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	float3 Result = float3(UNDEFINED_ACCURACY, UNDEFINED_ACCURACY, UNDEFINED_ACCURACY);

	if (CPULogDistance >= 0)
	{
		float ViewDistance = length(SvPositionToResolvedTranslatedWorld(DebugInputs.SvPosition));
		float GPULogDistance = log2(max(1, ViewDistance));

		float Accuracy = clamp(GPULogDistance - CPULogDistance, -1.99, 1.99);
		int ColorIndex = floor(Accuracy) + 2;

		Result = lerp(DebugViewModeStruct.AccuracyColors[ColorIndex].rgb, DebugViewModeStruct.AccuracyColors[ColorIndex + 1].rgb, frac(Accuracy));
	}

	return float4(Result, PrimitiveAlpha);
}

float4 VisualizeRequiredTextureResolution(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(DebugInputs, DebugInputs.SvPosition);

	MaterialParameters.TexCoordScalesParams.RequiredResolution = 0;
	MaterialParameters.TexCoordScalesParams.TexSample = 0;

	half3 BaseColor;
	{
		FPixelMaterialInputs PixelMaterialInputs;
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, DebugInputs.SvPosition, bIsFrontFace);

		// Sample material properties. The results are not used, but the calls to StoreTexCoordScale are still be made.
		BaseColor = GetMaterialBaseColorRaw(PixelMaterialInputs);
		half  Metallic = GetMaterialMetallicRaw(PixelMaterialInputs);
		half  Specular = GetMaterialSpecularRaw(PixelMaterialInputs);
		float Roughness = GetMaterialRoughnessRaw(PixelMaterialInputs);
		half3 Normal = GetMaterialNormalRaw(PixelMaterialInputs);
		half3 Emissive = GetMaterialEmissiveRaw(PixelMaterialInputs);
		half Opacity = GetMaterialOpacityRaw(PixelMaterialInputs);
#if MATERIALBLENDING_MASKED
		half Mask = GetMaterialMask(PixelMaterialInputs);
		clip(GetMaterialMask(PixelMaterialInputs));
#endif
		half4 SSData = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs);
		float Custom0 = GetMaterialCustomData0(PixelMaterialInputs);
		float Custom1 = GetMaterialCustomData1(PixelMaterialInputs);
		float MaterialAO = GetMaterialAmbientOcclusionRaw(PixelMaterialInputs);
		float PixelDepthOffset = GetMaterialPixelDepthOffset(PixelMaterialInputs);
#if CLEAR_COAT_BOTTOM_NORMAL && NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
		float3 BottomNormal = ClearCoatBottomNormal0(MaterialParameters);
#endif
	}

	float3 Result = float3(UNDEFINED_ACCURACY, UNDEFINED_ACCURACY, UNDEFINED_ACCURACY);
	Result *= saturate(Luminance(BaseColor));

	if (VisualizeMode == DVSM_RequiredTextureResolution)
	{
		if (MaterialParameters.TexCoordScalesParams.RequiredResolution > 0)
		{
			float Accuracy = log2(TextureResolution / MaterialParameters.TexCoordScalesParams.RequiredResolution);
			Result = AccuracyColorLookup(Accuracy);
			Result *= MaterialParameters.TexCoordScalesParams.TexSample;
		}
	}

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif

	return float4(Result, PrimitiveAlpha);
}

float4 VisualizeShaderComplexityAccumulate(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	float3 Result = float3(1, 0, 1);
	if (bOutputQuadOverdraw)
	{
		float3 FinalComplexity = NormalizedComplexity.xyz;

#if !(COMPILER_METAL) && (OUTPUT_QUAD_OVERDRAW)
		[branch]
		if (bShowQuadOverdraw && NormalizedComplexity.x > 0)
		{
			uint Coverage = ComputeQuadCoverage(DebugInputs.SvPosition.xy, DebugInputs.SvPrimitiveID, 24, false, false, 0);
			// The extra cost from quad overdraw is assumed to be linear.
			FinalComplexity.x *= 4.f / (float)(Coverage);
		}
#endif // !(COMPILER_METAL) && (OUTPUT_QUAD_OVERDRAW)

		// use the maximum range allowed for scene color
		Result = FinalComplexity;
	}
	else
	{
		Result = NormalizedComplexity.xyz;
	}

	// alpha channel needs to be 1 to have decals working where the framebuffer blend is setup to multiply with alpha
	return float4(Result, 1);
}

float4 ApplyDebugColor(in FDebugPSIn DebugInputs, in bool bIsFrontFace, in float3 DebugColor)
{
	float3 Result = DebugColor;

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(DebugInputs, DebugInputs.SvPosition);

	half3 BaseColor;
	half Opacity;
	{
		FPixelMaterialInputs PixelMaterialInputs;
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, DebugInputs.SvPosition, bIsFrontFace);

		// Sample material properties. The results are not used, but the calls to StoreTexCoordScale are still be made.
		BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
		Opacity = GetMaterialOpacity(PixelMaterialInputs);

		half3 Emissive = GetMaterialEmissive(PixelMaterialInputs);
#if MATERIALBLENDING_ADDITIVE
		Opacity = (.05 + .95 * Luminance(Emissive));
#else
		BaseColor += Emissive;
#endif

#if MATERIALBLENDING_MASKED
		clip(GetMaterialMask(PixelMaterialInputs));
#endif
	}

	Result *= (.05 + .95 * Luminance(BaseColor));

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif

	return float4(Result, Opacity);
}

float4 VisualizeLODColoration(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	float3 Color = DebugViewModeStruct.LODColors[LODIndex].rgb;
	return ApplyDebugColor(DebugInputs, bIsFrontFace, Color);
}

float4 VisualizeGPUSkinCache(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	return ApplyDebugColor(DebugInputs, bIsFrontFace, SkinCacheDebugColor);
}

float4 VisualizeStreamingDeficit(in FDebugPSIn DebugInputs, in bool bIsFrontFace)
{
	float3 Color = DebugViewModeStruct.LODColors[clamp(LODIndex, 0, NUM_LOD_COLORATION_COLORS - 1)].rgb;
	return ApplyDebugColor(DebugInputs, bIsFrontFace, Color);
}

#if !OUTPUT_PIXEL_DEPTH_OFFSET
#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
#else
#define PIXELSHADER_EARLYDEPTHSTENCIL
#endif

PIXELSHADER_EARLYDEPTHSTENCIL
void Main(
	in FDebugPSIn DebugInputs
	OPTIONAL_IsFrontFace,
	out float4 OutColor : SV_Target0,
	out uint OutDebugAux : SV_Target1
	)
{
	StereoSetupPS(DebugInputs.StereoInput);

	uint PrimitiveId = DebugInputs.PrimitiveId;
	uint InstanceId  = DebugInputs.InstanceId;

	// If we see pink something went wrong.
	float4 VisualizeColor = float4(1, 0, 1, 1);
	uint DebugAux = 0;
	bool bApplyDebugShading = false;

	BRANCH
	if (VisualizeMode == DVSM_ShaderComplexity ||
		VisualizeMode == DVSM_ShaderComplexityContainedQuadOverhead ||
		VisualizeMode == DVSM_ShaderComplexityBleedingQuadOverhead ||
		VisualizeMode == DVSM_QuadComplexity ||
		VisualizeMode == DVSM_LWCComplexity)
	{
		VisualizeColor = VisualizeShaderComplexityAccumulate(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_PrimitiveDistanceAccuracy)
	{
		VisualizeColor = VisualizePrimitiveDistanceAccuracy(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_MeshUVDensityAccuracy)
	{
		VisualizeColor = VisualizeMeshTexCoordSizeAccuracy(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_MaterialTextureScaleAccuracy ||
			 VisualizeMode == DVSM_OutputMaterialTextureScales)
	{
		VisualizeColor = VisualizeMaterialTexCoordScales(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_RequiredTextureResolution)
	{
		VisualizeColor = VisualizeRequiredTextureResolution(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_LODColoration)
	{
		VisualizeColor = VisualizeLODColoration(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_VisualizeGPUSkinCache)
	{
		VisualizeColor = VisualizeGPUSkinCache(DebugInputs, bIsFrontFace);
	}
	else if (VisualizeMode == DVSM_ShadowCasters)
	{
		if (PrimitiveBlendMode != BLEND_Opaque && PrimitiveBlendMode != BLEND_Masked)
		{
			// Translucent objects don't work very well in this visualization, so we turn them off
			// as they are non-shadowcasting in 99% of cases anyway, and their bounds will still show.
			// Ideally we'd render them as translucent, or cull only the non-shadowcasting ones.
			discard;
			OutColor = 0;
			OutDebugAux = 0;
			return;
		}
		DebugAux = InstanceId;
		VisualizeColor = float4(1, 1, 1, 1);
		bApplyDebugShading = true;
	}
	else if (VisualizeMode == DVSM_StreamingDeficit)
	{
		VisualizeColor = VisualizeStreamingDeficit(DebugInputs, bIsFrontFace);
	}

	if (bApplyDebugShading)
	{
		// Apply N dot V shading using face normal
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(DebugInputs.SvPosition);
		float3 WorldNormal = normalize(cross(ddx(TranslatedWorldPosition), ddy(TranslatedWorldPosition)));
		float3 ViewVector = normalize( TranslatedWorldPosition - View.TranslatedWorldCameraOrigin );
		VisualizeColor *= abs(dot(WorldNormal, ViewVector)) * 0.7 + 0.3;
	}

	OutColor = RETURN_COLOR(VisualizeColor);
	OutDebugAux = DebugAux;
}
