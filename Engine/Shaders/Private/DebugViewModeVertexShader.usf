// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugViewModeVertexShader.hlsl: Debug shader used for special viewmode that need to preserve the geometry shape.
=============================================================================*/

#define TEX_COORD_SCALE_ANALYSIS 1

#define SCENE_TEXTURES_DISABLED 1

#include "Common.ush"
#include "VirtualTextureCommon.ush"

struct FTexCoordScalesParams
{
	uint Dummy;
};

MaterialFloat StoreTexCoordScale(in out FTexCoordScalesParams Params, float2 UV, int TextureReferenceIndex)
{
	return 1.f;
}

MaterialFloat StoreTexSample(in out FTexCoordScalesParams Params, float4 C, int TextureReferenceIndex)
{
	return 1.f;
}

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "DebugViewModeCommon.ush"
// struct FDebugVSToPS

#define FDebugVSOutput FDebugVSToPS

#if VERTEXSHADER

void Main(
	FVertexFactoryInput Input,
	out FDebugVSOutput Output
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, Output.StereoOutput);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPos = VertexFactoryGetWorldPosition(Input, VFIntermediates);

	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPos.xyz, TangentToLocal);

	// Isolate instructions used for world position offset
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	{
		WorldPos.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, WorldPos.xyz);
	}

	{
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPos);
		Output.Position = INVARIANT(mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip));
	}

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPos.xyz, 1));
#endif
	Output.VertexColor = VertexParameters.VertexColor;

	float3x3 TangentToWorld = VertexParameters.TangentToWorld;
	Output.TangentToWorld0 = float3(TangentToWorld[0]);
	Output.TangentToWorld1 = float3(TangentToWorld[1]);
	Output.TangentToWorld2 = float3(TangentToWorld[2]);

#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
		Output.TexCoord23 = Output.TexCoord01 = VertexParameters.TexCoords[0].xyxy;
#else
		Output.TexCoord23 = Output.TexCoord01 = 0;
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Output.TexCoord01.zw = VertexParameters.TexCoords[1];
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		Output.TexCoord23.xy = VertexParameters.TexCoords[2];
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		Output.TexCoord23.zw = VertexParameters.TexCoords[3];
#endif

	FSceneDataIntermediates SceneData = VertexParameters.SceneData;
	Output.PrimitiveId = SceneData.PrimitiveId;
	Output.InstanceId = SceneData.InstanceId;
}

#endif // VERTEXSHADER
