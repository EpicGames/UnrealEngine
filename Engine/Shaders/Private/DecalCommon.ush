// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DecalCommon.ush: Common decal output logic.
=============================================================================*/

// DECAL_RENDERSTAGE is set by C++ from EDecalRenderStage
// DECAL_RENDERTARGETMODE is set by C++ from EDecalRenderTarget
// DECAL_RENDERTARGET_COUNT is set by C++

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "DBufferDecalShared.ush"

#if SUBSTRATE_ENABLED  
FSubstrateDBuffer DecalCommonOutput(
	inout FPixelShaderOut Out, 
	in FSubstratePixelHeader InHeader,
	in FSubstrateBSDF InBSDF,
	in float FadeOutWeight,
	in float CoverageOverBackground)
{
	FSubstrateDBuffer OutDBuffer = InHeader.SubstrateConvertToDBuffer(InBSDF);

	OutDBuffer.Coverage *= FadeOutWeight;

	float3 Color = OutDBuffer.Emissive;

#if DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_MOBILE
	// Cheap mobile forward approximation uses both emissive and color.
	Color += OutDBuffer.BaseColor;
#if MATERIALBLENDING_MODULATE
	Color *= OutDBuffer.Coverage;
#endif
#elif DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_AO
	// Replace color with AO for AO decals.
#if SUBSTRATE_INLINE_SHADING
	Color = lerp(1, InHeader.IrradianceAO.MaterialAO, OutDBuffer.Coverage);
#else
#error Substrate - Decals must be evaluated inline.
#endif
#endif

#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_SCENECOLOR
	Color *= View.PreExposure;
#if SUBSTRATE_USES_CONVERSION_FROM_LEGACY && MATERIALBLENDING_ALPHACOMPOSITE
	// In legacy mode, emissive is always influenced by opacity, even when AlphaComposite blend mode is used.
	// This is only here to maintain the legacy behavior after material are converted; not with Substrate itself.
	Color *= CoverageOverBackground;
#endif
#endif

	// Outputing color to MRT0 (can be emissive or any other data set above)
	Out.MRT[0] = float4(Color, OutDBuffer.Coverage);

#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_DBUFFER
	// If we output to OutDBuffer, write to all the 3 RTs.
	SubstratePackDBuffer(OutDBuffer, Out.MRT[0], Out.MRT[1], Out.MRT[2]);

	#if DECAL_RENDERTARGET_COUNT == 4
		Out.MRT[3] = 1.0f; // DBufferMask
	#endif

	// This is to set the coverage over the background to an arbitrary value in case premultiplied alpha is used.
	Out.MRT[0].a = CoverageOverBackground;
	Out.MRT[1].a = CoverageOverBackground;
	Out.MRT[2].a = CoverageOverBackground;
#endif

	// Allow shader compiler to cull outputs that we won't use.
	// Blend state also prevents writing unused outputs, but we want to be sure that we also avoid shader cost.
#if !DECAL_OUT_MRT0
	Out.MRT[0] = 0;
#endif
#if !DECAL_OUT_MRT1
	Out.MRT[1] = 0;
#endif
#if !DECAL_OUT_MRT2
	Out.MRT[2] = 0;
#endif
#if !DECAL_OUT_MRT3
	Out.MRT[3] = 0;
#endif

	return OutDBuffer;
}
#endif // SUBSTRATE_ENABLED

void DecalCommonOutput(inout FPixelShaderIn In, inout FPixelShaderOut Out, float3 Color, float Opacity, FGBufferData Data)
{
#if DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_MOBILE
	// Cheap mobile forward approximation uses both emissive and color.
	Color += Data.BaseColor;
	#if MATERIALBLENDING_MODULATE
		Color *= Opacity;
	#endif
#elif DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_AO
	// Replace color with AO for AO decals.
	Color = lerp(1, Data.GBufferAO, Opacity);
#endif

	#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_SCENECOLOR || DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER || DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER_NONORMAL
		Color *= View.PreExposure;
	#endif

	Out.MRT[0] = float4(Color, Opacity);

#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_DBUFFER
	{
		float3 MultiOpacity = Opacity;

		EncodeDBufferData(Data, MultiOpacity, Out.MRT[0], Out.MRT[1], Out.MRT[2]);

		#if DECAL_RENDERTARGET_COUNT == 4
			Out.MRT[3] = 1.0f; // DBufferMask
		#endif
	}
#endif

// This is not support by Substrate
#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER || DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER_NONORMAL
	
	// Dummy outputs from EncodeGBuffer()
	float4 OutTarget1 = 0;
	float4 OutTarget4 = 0;
	float4 OutTarget5 = 0;
	float4 OutTarget6 = 0;

	#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER
	{
		EncodeGBuffer(Data, Out.MRT[1], Out.MRT[2], Out.MRT[3], OutTarget4, OutTarget5, OutTarget6);
	}
	#elif DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER_NONORMAL
	{
		EncodeGBuffer(Data, OutTarget1, Out.MRT[1], Out.MRT[2], OutTarget4, OutTarget5, OutTarget6);
	}
	#endif // DECAL_RENDERTARGETMODE

	#if MATERIALBLENDING_MODULATE
		// BaseColor MRT location changes for render target mode.
		#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER
			Out.MRT[3].rgb *= Opacity;
		#elif DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_GBUFFER_NONORMAL
			Out.MRT[2].rgb *= Opacity;
		#endif
	#endif // MATERIALBLENDING_MODULATE

	Out.MRT[1].a = Opacity;
	Out.MRT[2].a = Opacity;
	Out.MRT[3].a = Opacity;

#endif // DECAL_RENDERTARGETMODE

	// Allow shader compiler to cull outputs that we won't use.
	// Blend state also prevents writing unused outputs, but we want to be sure that we also avoid shader cost.
#if !DECAL_OUT_MRT0
	Out.MRT[0] = 0;
#endif
#if !DECAL_OUT_MRT1
	Out.MRT[1] = 0;
#endif
#if !DECAL_OUT_MRT2
	Out.MRT[2] = 0;
#endif
#if !DECAL_OUT_MRT3
	Out.MRT[3] = 0;
#endif
}
