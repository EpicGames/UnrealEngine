// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "DeferredShadingCommon.ush"  
#include "BRDF.ush"
#include "ScreenSpaceReflectionTileCommons.ush"

#include "Substrate/Substrate.ush"
#include "Substrate/SubstrateEvaluation.ush"

// Tiling for pixels that require SSR

#ifndef TILE_CATERGORISATION_SHADER
#define TILE_CATERGORISATION_SHADER 1
#endif

#if TILE_CATERGORISATION_SHADER

#if USE_SSR_PRE_PASS_STENCIL
Texture2D<uint2> SSRDepthStencilTexture;
#endif

float MaxRoughness;
float MinSpecular;
int bEnableTwoSidedFoliage;

int2 TiledViewRes;
RWStructuredBuffer<uint> TileMaskBufferOut;

#if COMPILER_SUPPORTS_WAVE_VOTE
groupshared uint bAnySSRPixels;
#else
groupshared bool ContainsSSR[SSR_TILE_SIZE_XY * SSR_TILE_SIZE_XY];
#endif

bool DoesPixelContainSSR(uint2 PixelPos)
{
#if USE_SSR_PRE_PASS_STENCIL
	uint Stencil = SSRDepthStencilTexture.Load(uint3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
	return (Stencil & 1U) != 0U;
#else // !USE_SSR_PRE_PASS_STENCIL
#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(SubstratePixelHeader.PackedTopLayerData);

	bool bRoughnessIsValid = TopLayerData.Roughness <= MaxRoughness;
	bool bContainValidPixels = SubstratePixelHeader.ClosureCount > 0 && bRoughnessIsValid;
	BRANCH
	if (SubstratePixelHeader.ClosureCount == 1) // Only non complex materials will be interpreted for exclusion for now when it comes to Specular values or legacy shading models.
	{
		const bool bSkipSSSMaterialOverride = true; // We do not want the material data to be affected by any override.
		FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader, bSkipSSSMaterialOverride);

		const float BSDFSpecular = F0ToDielectricSpecular(max3(SLAB_F0(BSDF).x, SLAB_F0(BSDF).y, SLAB_F0(BSDF).z));

		bContainValidPixels =
			bRoughnessIsValid
			&& BSDFSpecular >= MinSpecular
			&& !SubstratePixelHeader.IsHair()
			&& !SubstratePixelHeader.IsSingleLayerWater();

		bContainValidPixels = bContainValidPixels && (bEnableTwoSidedFoliage ? true : (BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_TWO_SIDED_WRAP));

		const uint BSDFType = BSDF_GETTYPE(BSDF);
		switch (BSDFType)
		{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			// Cull special case where specular=0, and roughness=0
			if (SLAB_ROUGHNESS(BSDF) == 0 && BSDFSpecular == 0)
			{
				bContainValidPixels = false;
			}

			// Clearcoat: if the material is clear coat it contains valid pixel.
			const bool bHaziness = BSDF_GETHASHAZINESS(BSDF);
			if (bHaziness)
			{
				const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
				const bool bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;
				bContainValidPixels = bContainValidPixels || bHazeAsSimpleClearCoat;
			}
			break;
		}
		}
	}

	return bContainValidPixels;
#else
	float2 BufferUV = (PixelPos + 0.5f) * View.BufferSizeAndInvSize.zw;
	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);

	bool bContainValidPixels = 
		GBuffer.Roughness <= MaxRoughness
		&& GBuffer.Specular >= MinSpecular
		&& (GBuffer.ShadingModelID != SHADINGMODELID_UNLIT)
		&& (GBuffer.ShadingModelID != SHADINGMODELID_HAIR)
		&& (GBuffer.ShadingModelID != SHADINGMODELID_SINGLELAYERWATER);

	// Ignore two sided foliage 
	bContainValidPixels = bContainValidPixels && (bEnableTwoSidedFoliage ? true : (GBuffer.ShadingModelID != SHADINGMODELID_TWOSIDED_FOLIAGE));

	// Cull special case where specular=0, and roughness=0
	if (GBuffer.Roughness == 0 && GBuffer.Specular == 0)
	{
		bContainValidPixels = false;
	}

	// Clearcoat: if the material is clear coat it contains valid pixel.
	bContainValidPixels = bContainValidPixels || (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT);

	return bContainValidPixels;
#endif
#endif // USE_SSR_PRE_PASS_STENCIL
}

/**
 * Check all GBuffer pixels and set 1 bit for any 8x8 tile which contains a water pixel
 */
[numthreads(SSR_TILE_SIZE_XY, SSR_TILE_SIZE_XY, 1)]
void SSRTileCategorisationMarkCS(uint3 ThreadId : SV_DispatchThreadID, uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
#if 0
	// Slow reference path
	const int LinearIndex = GroupThreadId.y * SSR_TILE_SIZE_XY + GroupThreadId.x;
	if (LinearIndex < 1)
	{
		bool bContainsSSR = false;
		for (uint i = 0; i < SSR_TILE_SIZE_XY; ++i)
		{
			for (uint j = 0; j < SSR_TILE_SIZE_XY; ++j)
			{
				bContainsSSR = bContainsSSR || DoesPixelContainSSR((ThreadId.xy + uint2(i, j)) + View.ViewRectMin.xy);
			}
		}
		if (bContainsSSR)
		{
			uint WriteToIndex;
			InterlockedAdd(DrawIndirectDataUAV[1], 1, WriteToIndex);
			InterlockedAdd(DispatchIndirectDataUAV[0], 1);
			// Encoding needs to match Lumen reflection tile encoding (see LumenReflection.usf)
			SSRTileListDataUAV[WriteToIndex] = PackTileCoord12bits(GroupId.xy);
		}
	}

#else

	bool bContainsSSR = false;
	if (ThreadId.x < uint(View.BufferSizeAndInvSize.x) && ThreadId.y < uint(View.BufferSizeAndInvSize.y))
	{
		bContainsSSR = bContainsSSR || DoesPixelContainSSR(ThreadId.xy + View.ViewRectMin.xy);
	}

	bool bWriteTile = false;

#if COMPILER_SUPPORTS_WAVE_VOTE

	if (all(GroupThreadId == 0))
	{
		bAnySSRPixels = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	const bool bAnySSRPixelsInWave = WaveActiveAnyTrue(bContainsSSR);
	if (WaveIsFirstLane() && bAnySSRPixelsInWave)
	{
		InterlockedAdd(bAnySSRPixels, 1);
	}
	GroupMemoryBarrierWithGroupSync();

	if (all(GroupThreadId == 0))
	{
		bWriteTile = bAnySSRPixels > 0;
	}

#else

	const int LinearIndex = GroupThreadId.y * SSR_TILE_SIZE_XY + GroupThreadId.x;
	ContainsSSR[LinearIndex] = bContainsSSR;
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 32) // 8*8 = 64 elements to merge
	{
		ContainsSSR[LinearIndex] = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		ContainsSSR[LinearIndex] = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	// The smallest wave size is 16 on Intel hardware. So now we can do simple math operations without group sync.
	// EDIT: for some reason group sync is needed until the end, otherwise some pixels are missing...

	if (LinearIndex < 8)
	{
		ContainsSSR[LinearIndex] = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		ContainsSSR[LinearIndex] = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		ContainsSSR[LinearIndex] = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 1)
	{
		bWriteTile = ContainsSSR[LinearIndex] || ContainsSSR[LinearIndex + 1];
	}

#endif

	if (bWriteTile)
	{
		// Set bit to indicate tile is occupied.
		uint MaskLinearIndex = TiledViewRes.x * GroupId.y + GroupId.x;
		InterlockedOr(TileMaskBufferOut[MaskLinearIndex / 32U], 1U << (MaskLinearIndex % 32U));
	}
#endif
}

uint VertexCountPerInstanceIndirect;
RWBuffer<uint> DrawIndirectDataUAV;
RWBuffer<uint> DispatchIndirectDataUAV;
RWBuffer<uint> SSRTileListDataUAV;
StructuredBuffer<uint> TileMaskBuffer;

groupshared uint SharedNumTiles;
groupshared uint SharedTileData[SSR_TILE_SIZE_XY * SSR_TILE_SIZE_XY];
groupshared uint SharedGlobalTileOffset;

/**
 * Every group checks 64 tiles and builds a spatially coherent compacted list of SSR tiles
 */
[numthreads(SSR_TILE_SIZE_XY, SSR_TILE_SIZE_XY, 1)]
void SSRTileClassificationBuildListsCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID)
{
	const uint LinearThreadIndex = GroupThreadId.y * SSR_TILE_SIZE_XY + GroupThreadId.x;

	if (all(DispatchThreadId == 0))
	{
		// TODO compute clear myself
		DrawIndirectDataUAV[0] = VertexCountPerInstanceIndirect;		// VertexCountPerInstance
		//DrawIndirectDataUAV[1]		// InstanceCount			already cleared to 0
		//DrawIndirectDataUAV[2] = 0;	// StartVertexLocation			"			"
		//DrawIndirectDataUAV[3] = 0;	// StartInstanceLocation		"			"

		DispatchIndirectDataUAV[1] = 1;
		DispatchIndirectDataUAV[2] = 1;
	}

	if (LinearThreadIndex == 0)
	{
		SharedNumTiles = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	//@todo - parallel version
	if (LinearThreadIndex == 0)
	{
		SharedNumTiles = 0;

		for (uint LocalTileIndex = 0; LocalTileIndex < SSR_TILE_SIZE_XY * SSR_TILE_SIZE_XY; ++LocalTileIndex)
		{
			// ZOrder tiles to maximize screen locality after converting to 1d for compaction
			// The tile locality ultimately affects trace coherency, since trace compaction pulls from neighboring tiles
			uint2 ThreadOffset = ZOrder2D(LocalTileIndex, log2(SSR_TILE_SIZE_XY));
			uint2 TileCoordinate = GroupId * SSR_TILE_SIZE_XY + ThreadOffset;

			if (all(TileCoordinate < TiledViewRes))
			{
				uint MaskLinearIndex = TiledViewRes.x * TileCoordinate.y + TileCoordinate.x;
				uint Mask = 1u << (MaskLinearIndex % 32u);
				bool bTileUsed = (TileMaskBuffer[MaskLinearIndex / 32u] & Mask) != 0;
				if (bTileUsed)
				{
					uint TileOffset = SharedNumTiles;
					// Encoding needs to match Lumen reflection tile encoding (see LumenReflection.usf)
					SharedTileData[TileOffset] = PackTileCoord12bits(TileCoordinate);
					SharedNumTiles = TileOffset + 1;
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Allocate space in the tile list
	if (LinearThreadIndex == 0 && SharedNumTiles > 0)
	{
		InterlockedAdd(DrawIndirectDataUAV[1], SharedNumTiles, SharedGlobalTileOffset);
		InterlockedAdd(DispatchIndirectDataUAV[0], SharedNumTiles);
	}

	GroupMemoryBarrierWithGroupSync();

	// Write out tiles
	if (LinearThreadIndex < SharedNumTiles)
	{
		SSRTileListDataUAV[SharedGlobalTileOffset + LinearThreadIndex] = SharedTileData[LinearThreadIndex];
	}
}

#endif