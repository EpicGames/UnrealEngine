// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredDecal.usf: Pixel shader for computing a deferred decal.
=============================================================================*/

#include "Common.ush"

#if SHADING_PATH_MOBILE
#if DECAL_RENDERTARGETMODE == DECAL_RENDERTARGETMODE_DBUFFER
	#define MobileSceneTextures DecalPass.MobileSceneTextures
	#define EyeAdaptationStruct DecalPass
	#define SubstrateStruct DecalPass.SubstratePublic
#else
	// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer
	#define MobileSceneTextures MobileBasePass.SceneTextures
	#define EyeAdaptationStruct MobileBasePass
	#define SubstrateStruct MobileBasePass.SubstratePublic
#endif
#else
#define SceneTexturesStruct DecalPass.SceneTextures
#define EyeAdaptationStruct DecalPass
#define SubstrateStruct DecalPass.SubstratePublic
#endif

#if PIXELSHADER

// Decals require both inline & deferred shading path for reading & writing Substrate data
#define SUBSTRATE_INLINE_SHADING 1
#define SUBSTRATE_DEFERRED_SHADING 1
#include "Substrate/Substrate.ush"
#include "DecalCommon.ush"

#if DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_MOBILE
#include "MobileLightingCommon.ush"
#endif

#endif // PIXELSHADER

// large world coordinate tile position
float4 DecalPositionHigh;
// from SvPosition to decal space (for position), used like this: mul(float4(SvPosition.xyz, 1), SvPositionToDecal);
float4x4 SvPositionToDecal;
#if MOBILE_MULTI_VIEW
// For stereo rendering.
float4x4 RightEyeSvPositionToDecal;
#endif
// to transform position from decal space to world space, for normals use transpose of inverse
float4x4 DecalToWorld;
float3 DecalToWorldInvScale;
// decal orientation in world space, X axis. Used for ObjectOrientation material expression
float3 DecalOrientation;
// x - Fade alpha from screen size fading
// y - Opacity over the lifetime of the decal. 1 -> 0
float2 DecalParams;
// from component to clip space (for decal frustum)
float4x4 FrustumComponentToClip;
// Decal color can be accessed using the material node
float4 DecalColorParam;

// Global decal vertex shader
void MainVS(
	in FStereoVSInput StereoInput,
	in float4 InPosition : ATTRIBUTE0,
	out float4 OutPosition : SV_POSITION,
	out FStereoVSOutput StereoOutput
)
{
	StereoSetupVS(StereoInput, StereoOutput);
	
#if MOBILE_MULTI_VIEW
	float4x4 FrustumToClip = mul(FrustumComponentToClip, ResolvedView.MobileMultiviewDecalTransform);
#else
	float4x4 FrustumToClip = FrustumComponentToClip;
#endif
	OutPosition = mul(InPosition, FrustumToClip);
}

#if IS_DECAL && PIXELSHADER

// DECAL_PRIMITIVE informs material templates which functions to expose when rendering decals.
#define DECAL_PRIMITIVE 1
#include "/Engine/Generated/Material.ush"

#define MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER (SUBSTRATE_ENABLED && (DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_MOBILEBEFORELIGHTING || SUBSTRATE_GBUFFER_FORMAT==0))

#if MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#define SUBSTRATE_MATERIAL_EXPORT_REQUESTED 1
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

// If virtual texture feedback is enabled then use early depth test so that UAV feedback buffer writes respect the depth test
#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	#if COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE
		// If we support early depth test with late write behaviour then use it since we may be using discard, or modifying depth
		#define PIXELSHADER_EARLYDEPTHSTENCIL DEPTHSTENCIL_EARLYTEST_LATEWRITE
	#elif !OUTPUT_PIXEL_DEPTH_OFFSET
		// Otherwise we can only use early depth test if not modifying depth
		// Modifying depth will trigger the slow path where we write feedback to UAV even where depth occluded!
		#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
	#endif
#endif

float4 SvPositionToScreenPosition2(float4 SvPosition)
{
	// assumed SvPosition.w = 1

	float4 Ret;

	Ret.x = ((SvPosition.x - ResolvedView.ViewRectMin.x) * ResolvedView.ViewSizeAndInvSize.z) * 2 - 1;
	Ret.y = ((SvPosition.y - ResolvedView.ViewRectMin.y) * ResolvedView.ViewSizeAndInvSize.w) * -2 + 1;
	Ret.z = ConvertFromDeviceZ(SvPosition.z);
	Ret.w = 1;

	Ret.xy *= Ret.z;

	// so .w has the SceneDepth, some mobile code wants that
//	Ret *= Ret.z;

	return Ret;
}

// is called in MainPS() from PixelShaderOutputCommon.usf
void FPixelShaderInOut_MainPS(inout FPixelShaderIn In, inout FPixelShaderOut Out, uint ArrayIndex)
{
	float2 ScreenUV = SvPositionToBufferUV(In.SvPosition);

	// make SvPosition appear to be rasterized with the depth from the depth buffer
#if MOBILE_MULTI_VIEW
	In.SvPosition.z = LookupDeviceZ(ScreenUV, ArrayIndex);
	float4x4 SvPositionToDecalForSlice = ArrayIndex ? RightEyeSvPositionToDecal : SvPositionToDecal;
#else
	In.SvPosition.z = LookupDeviceZ(ScreenUV);
	float4x4 SvPositionToDecalForSlice = SvPositionToDecal;
#endif
	In.SvPosition.w = ConvertFromDeviceZ(In.SvPosition.z);

	float4 DecalVectorHom = mul(float4(In.SvPosition.xyz,1), SvPositionToDecalForSlice);
	float3 OSPosition = DecalVectorHom.xyz / DecalVectorHom.w;

	// clip content outside the decal
	// not needed if we stencil out the decal but we do that only on large (screen space) ones
	clip(OSPosition.xyz + 1.0f);
	clip(1.0f - OSPosition.xyz);

	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(In.SvPosition);

	float3 CameraVector = normalize(ResolvedView.TranslatedWorldCameraOrigin - TranslatedWorldPosition);

	// can be optimized
	float3 DecalVector = OSPosition * 0.5f + 0.5f;

	// Swizzle so that DecalVector.xy are perpendicular to the projection direction and DecalVector.z is distance along the projection direction
	float3 SwizzlePos = DecalVector.zyx;

	// By default, map textures using the vectors perpendicular to the projection direction
	float2 DecalUVs = SwizzlePos.xy;

	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
#if NUM_MATERIAL_TEXCOORDS
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_MATERIAL_TEXCOORDS;CoordinateIndex++)
	{
		MaterialParameters.TexCoords[CoordinateIndex] = DecalUVs;
	}
#endif
	MaterialParameters.TwoSidedSign = 1;
	MaterialParameters.VertexColor = 1;
	MaterialParameters.CameraVector = CameraVector;
	MaterialParameters.SvPosition = In.SvPosition;
	MaterialParameters.ScreenPosition = SvPositionToResolvedScreenPosition(In.SvPosition);
	MaterialParameters.LightVector = SwizzlePos;

	MaterialParameters.AbsoluteWorldPosition = MaterialParameters.WorldPosition_NoOffsets = DFFastSubtract(TranslatedWorldPosition, PrimaryView.PreViewTranslation);
	MaterialParameters.WorldPosition_CamRelative = MaterialParameters.WorldPosition_NoOffsets_CamRelative = TranslatedWorldPosition;
	MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	InitializeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback);
#endif

	FPixelMaterialInputs PixelMaterialInputs;
	CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
	
	const float DecalFading = saturate(4 - 4 * abs(SwizzlePos.z * 2 - 1)) * DecalParams.x;

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	FSubstratePixelHeader SubstrateHeader = MaterialParameters.GetFrontSubstrateHeader();
	const float MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
	SubstrateHeader.IrradianceAO.MaterialAO = MaterialAO;

#if MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER

	float3 EmissiveColor = 0.0f.xxx;
	float Opacity = 0.0f;
	FGBufferData DBufferData = (FGBufferData)0;
	if (SubstrateHeader.SubstrateTree.BSDFCount > 0)
	{
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, false/*bRoughnessTracking*/);
		const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
		FExportResult Export = SubstrateMaterialExportOut(
			Settings,
			SubstrateHeader,
			PixelMaterialInputs.GetFrontSubstrateData(),
			CameraVector,
			SurfaceWorldNormal,
			MaterialParameters.WorldPosition_CamRelative,
			0.0f /*MobileShadingPathCurvature*/);
		
		// Either propagate the BSDF opacity, or use the one for AlphaComposite (premultiplied alpha)
		float CoverageOverBackground = Export.Coverage * DecalFading;
	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		CoverageOverBackground = GetMaterialOpacity(PixelMaterialInputs) * DecalFading;
	#endif 
		Opacity										= CoverageOverBackground;
		EmissiveColor								= Export.EmissiveLuminance;

		DBufferData.WorldNormal						= Export.WorldNormal;
		DBufferData.BaseColor						= Export.BaseColor;
		DBufferData.Metallic						= Export.Metallic;
		DBufferData.Specular						= Export.Specular;
		DBufferData.Roughness						= Export.Roughness;
		DBufferData.CustomData						= 0;
		DBufferData.IndirectIrradiance				= 0;
		DBufferData.PrecomputedShadowFactors		= 1;
		DBufferData.GBufferAO						= MaterialAO;
		DBufferData.ShadingModelID					= Export.ShadingModelID;
		DBufferData.SelectiveOutputMask				= 0;
		DBufferData.PerObjectGBufferData			= 1;
		DBufferData.DiffuseIndirectSampleOcclusion	= 0;
		DBufferData.Velocity						= 0;

		DecalCommonOutput(In, Out, EmissiveColor, Opacity, DBufferData);
	}

#else

	float Coverage = 0.0;
	FSubstrateDBuffer DBufferData =(FSubstrateDBuffer)0;
	if (SubstrateHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;

		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, false/*bRoughnessTracking*/);
		SubstrateHeader.SubstrateUpdateTree(V, Settings);

		// Either propagate the BSDF opacity, or use the one for AlphaComposite (premultiplied alpha)
		float CoverageOverBackground = SubstrateHeader.SubstrateTree.BSDFs[0].Coverage * DecalFading;
	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		CoverageOverBackground = GetMaterialOpacity(PixelMaterialInputs) * DecalFading;
	#endif 

		// Since the Convert-To-Decal node forces parameter blending for the entire Substrate tree, the BSDF we are interested in will be in 0.
		DBufferData = DecalCommonOutput(Out, SubstrateHeader, SubstrateHeader.SubstrateTree.BSDFs[0], DecalFading, CoverageOverBackground);
	}
	const float Opacity = DBufferData.Coverage;
	const float3 EmissiveColor = DBufferData.Emissive;

#endif

#else // SUBSTRATE_ENABLED

	float3 EmissiveColor = GetMaterialEmissive(PixelMaterialInputs);
	
	float Opacity = GetMaterialOpacity(PixelMaterialInputs) * DecalFading;

	FGBufferData DBufferData;
	DBufferData.WorldNormal = MaterialParameters.WorldNormal;
	DBufferData.BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	DBufferData.Metallic = GetMaterialMetallic(PixelMaterialInputs);
	DBufferData.Specular = GetMaterialSpecular(PixelMaterialInputs);
	DBufferData.Roughness = GetMaterialRoughness(PixelMaterialInputs);
	DBufferData.CustomData = 0;
	DBufferData.IndirectIrradiance = 0;
	DBufferData.PrecomputedShadowFactors = 1;
	DBufferData.GBufferAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
	DBufferData.ShadingModelID = SHADINGMODELID_DEFAULT_LIT;
	DBufferData.SelectiveOutputMask = 0;
	DBufferData.PerObjectGBufferData = 1; 
	DBufferData.DiffuseIndirectSampleOcclusion = 0;
	DBufferData.Velocity = 0;

	DecalCommonOutput(In, Out, EmissiveColor, Opacity, DBufferData);
#endif // SUBSTRATE_ENABLED

#if DECAL_RENDERSTAGE == DECAL_RENDERSTAGE_MOBILE
	#if DECAL_MOBILE_FORWARD_LIT
		Out.MRT[0].rgb = ForwardDecalLighting(
			TranslatedWorldPosition, 
			MaterialParameters.ScreenPosition.xy, 
			MaterialParameters.ScreenPosition.w,
			CameraVector, 
			MaterialParameters.TangentToWorld[2], 
			ReflectionAboutCustomWorldNormal(MaterialParameters, MaterialParameters.WorldNormal, false),
			DBufferData.WorldNormal,
			DBufferData.Specular,
			DBufferData.BaseColor,
			DBufferData.Metallic,
			DBufferData.Roughness,
			EmissiveColor,
			Opacity);
	#endif
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif
}

void FPixelShaderInOut_MainPS(inout FPixelShaderIn In, inout FPixelShaderOut Out)
{
	FPixelShaderInOut_MainPS(In, Out, 0);
}

//#define PIXELSHADEROUTPUT_MRT0 (DECAL_RENDERTARGET_COUNT > 0)
//#define PIXELSHADEROUTPUT_MRT1 (DECAL_RENDERTARGET_COUNT > 1)
//#define PIXELSHADEROUTPUT_MRT2 (DECAL_RENDERTARGET_COUNT > 2)
//#define PIXELSHADEROUTPUT_MRT3 (DECAL_RENDERTARGET_COUNT > 3)
//#define PIXELSHADEROUTPUT_MRT4 (DECAL_RENDERTARGET_COUNT > 4)
// all PIXELSHADEROUTPUT_ and "void FPixelShaderInOut_MainPS()" need to be setup before this include
// this include generates the wrapper code to call MainPS()
#include "PixelShaderOutputCommon.ush"

#endif // IS_DECAL
