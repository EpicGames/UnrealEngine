// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredLightPixelShaders.usf:
=============================================================================*/

#pragma message("UESHADERMETADATA_VERSION 1543FD56-DAD0-4ACF-ADFE-7B1543694F24")

#if USE_HAIR_LIGHTING == 1
  #define SUPPORT_CONTACT_SHADOWS 0
#else
  #define SUPPORT_CONTACT_SHADOWS 1
#endif
#define SUBSTRATE_SSS_TRANSMISSION USE_TRANSMISSION

#if SUBSTRATE_ENABLED 
  #if SUBSTRATE_TILETYPE == 0
	#define SUBSTRATE_FASTPATH 1
  #elif SUBSTRATE_TILETYPE == 1
	#define SUBSTRATE_SINGLEPATH 1
  #elif SUBSTRATE_TILETYPE == 2
	// COMPLEX PATH
  #elif SUBSTRATE_TILETYPE == 3
	// COMPLEX SPECIAL PATH
	#define SUBSTRATE_COMPLEXSPECIALPATH 1
  #else
	#error Substrate tile type non-implemented
  #endif
#endif

#define SUBSTRATE_OUTPUT_TOON_DATA (SUBSTRATE_ENABLED && SUBSTRATE_TILETYPE >= 2 && SUBSTRATE_EXPERIMENTAL_TOON_UNIFIED_DIFFUSE)
#define SUBSTRATE_LIGHTPASS_APPLIES_TOON_DIFFUSE_RAMP (SUBSTRATE_OUTPUT_TOON_DATA ? 0 : 1)

#if SUBSTRATE_GBUFFER_FORMAT==0 || USE_HAIR_LIGHTING
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 0	// Load from a specified BSDF, not from MaterialBuffer.
#define SUBSTRATE_INLINE_SHADING 1				// Inline shading for normal setup if needed.
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 1		// Need for SSS dither.
#include "Substrate/SubstrateRead.ush"
#else 
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 1
#endif

//#include "HeterogeneousVolumes/HeterogeneousVolumesVoxelGridTypes.ush"
#include "HairStrands/HairStrandsVisibilityCommonStruct.ush" 
#include "Common.ush"

#include "LargeWorldCoordinates.ush"

#include "Substrate/Substrate.ush"
#include "DeferredShadingCommon.ush"
#include "DeferredLightingCommon.ush"
#include "LightDataUniforms.ush"

#if USE_HAIR_LIGHTING == 1
#include "HairStrands/HairStrandsVisibilityCommon.ush"
#include "HairStrands/HairStrandsVisibilityUtils.ush"
#endif
#include "HairStrands/HairStrandsCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceDualScattering.ush" 

#if USE_ATMOSPHERE_TRANSMITTANCE
#include "/Engine/Private/SkyAtmosphereCommon.ush"
#endif

#if USE_CLOUD_TRANSMITTANCE
#include "VolumetricCloudCommon.ush"
#endif

#include "Substrate/SubstrateEvaluation.ush"
#include "Substrate/SubstrateDeferredLighting.ush"

#if USE_VIRTUAL_SHADOW_MAP_MASK
#include "VirtualShadowMaps/VirtualShadowMapMaskBitsCommon.ush"
#endif

#include "PostProcessSubsurfaceDiffuse.ush"

// Remap light shape permutation into light type
#if LIGHT_SOURCE_SHAPE == 0
#define CURRENT_LIGHT_TYPE LIGHT_TYPE_DIRECTIONAL
#elif LIGHT_SOURCE_SHAPE == 2
#define CURRENT_LIGHT_TYPE LIGHT_TYPE_RECT
#else // LIGHT_SOURCE_SHAPE == 1
#define CURRENT_LIGHT_TYPE (IsDeferredSpotlight() ? LIGHT_TYPE_SPOT : LIGHT_TYPE_POINT)
#endif

struct FInputParams
{
	float2 PixelPos;
	float4 ScreenPosition;
	float2 ScreenUV;
	float3 ScreenVector;
};

struct FDerivedParams
{
	float3 CameraVector;
	float3 TranslatedWorldPosition;
};

FDerivedParams GetDerivedParams(in FInputParams Input, in float SceneDepth)
{
	FDerivedParams Out;
#if LIGHT_SOURCE_SHAPE > 0
	// With a perspective projection, the clip space position is NDC * Clip.w
	// With an orthographic projection, clip space is the same as NDC
	float2 ClipPosition = GetScreenPositionForProjectionType(Input.ScreenPosition.xy / Input.ScreenPosition.w, SceneDepth);
	Out.TranslatedWorldPosition = mul(float4(ClipPosition, SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
	Out.CameraVector = GetCameraVectorFromTranslatedWorldPosition(Out.TranslatedWorldPosition);
#else
	Out.TranslatedWorldPosition = Input.ScreenVector * SceneDepth + GetTranslatedWorldCameraPosFromView(Input.PixelPos);
	Out.CameraVector = normalize(Input.ScreenVector);
#endif
	return Out;
}

Texture2D<uint> LightingChannelsTexture;

uint bLightingChannelEnabled;
uint GetLightingChannelMask(float2 UV)
{
	uint2 IntegerUV = UV * View.BufferSizeAndInvSize.xy;
	uint Out = 0xFFu;
	IF_UBERSHADER(bLightingChannelEnabled)
	{
		Out = LightingChannelsTexture.Load(uint3(IntegerUV, 0)).x;
	}
	return Out;
}

float GetExposure()
{
	return View.PreExposure;
}

#if USE_VIRTUAL_SHADOW_MAP_MASK
// One pass projection
Texture2D<uint4> ShadowMaskBits;
int VirtualShadowMapId;
int LightSceneId;
#endif

float4 GetLightAttenuationFromShadow(in FInputParams InputParams, float SceneDepth, float3 TranslatedWorldPosition)
{
	float4 LightAttenuation = float4(1, 1, 1, 1);

#if USE_VIRTUAL_SHADOW_MAP_MASK
	if (VirtualShadowMapId != INDEX_NONE)
	{
		float ShadowMask = GetVirtualShadowMapMaskForLight(
			ShadowMaskBits[InputParams.PixelPos],
			uint2(InputParams.PixelPos),
			SceneDepth,
			VirtualShadowMapId,
			TranslatedWorldPosition);
		return ShadowMask.xxxx;
	}
	else
#endif
	{
		return GetPerPixelLightAttenuation(InputParams.ScreenUV);
	}
}


////////////////////////////////////////////////////////////////////////////////////////////////////

#if USE_ATMOSPHERE_TRANSMITTANCE

float3 GetAtmosphericLightTransmittance(in float3 StableTranslatedLargeWorldPosition, in float2 ScreenUV, in float3 LightDirection)
{
	// The following is the only way to retrieve a world space position that is always stable under camera movement + long view distance. (DerivedParams.WorldPosition is not stable)
	const float3 PlanetCenterToWorldPos = (StableTranslatedLargeWorldPosition - View.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;

	const float3 AtmosphereTransmittance = GetAtmosphereTransmittance(
		PlanetCenterToWorldPos, LightDirection, View.SkyAtmosphereBottomRadiusKm, View.SkyAtmosphereTopRadiusKm,
		View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);

	return AtmosphereTransmittance;
}

#endif // USE_ATMOSPHERE_TRANSMITTANCE

uint bCloudShadowEnabled;
uint bAtmosphereTransmittanceEnabled;
float3 GetAtmosphereAndCloudAttenuation(float3 StableTranslatedLargeWorldPosition, float2 ScreenUV)
{
	float3 Out = 1;

	#if USE_ATMOSPHERE_TRANSMITTANCE
	IF_UBERSHADER(bAtmosphereTransmittanceEnabled)
	{
		Out *= GetAtmosphericLightTransmittance(StableTranslatedLargeWorldPosition, ScreenUV, DeferredLightUniforms.Direction.xyz);
	}
	#endif

	#if USE_CLOUD_TRANSMITTANCE
	IF_UBERSHADER(bCloudShadowEnabled)
	{
		float OutOpticalDepth = 0.0f;
		Out *= lerp(1.0f, GetCloudVolumetricShadow(StableTranslatedLargeWorldPosition, CloudShadowmapTranslatedWorldToLightClipMatrix, CloudShadowmapFarDepthKm, CloudShadowmapTexture, CloudShadowmapSampler, OutOpticalDepth), CloudShadowmapStrength);
	}
	#endif

	return Out;
}

float3 GetStableTranslatedLargeWorldPosition(float2 SVPos, float DeviceZ)
{
	float3 StableTranslatedLargeWorldPosition = 0;
	#if USE_ATMOSPHERE_TRANSMITTANCE || USE_CLOUD_TRANSMITTANCE
	StableTranslatedLargeWorldPosition = SvPositionToTranslatedWorld(float4(SVPos, DeviceZ, 1.0));
	#endif
	return StableTranslatedLargeWorldPosition;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if SUBSTRATE_ENABLED

float4 SubstrateReadPrecomputedShadowFactors(FSubstratePixelHeader SubstratePixelHeader, int2 PixelPos)
{
	if (SubstratePixelHeader.HasPrecShadowMask())
	{
#if ALLOW_STATIC_LIGHTING
		float4 GBufferE = SceneTexturesStruct.GBufferETexture.Load(int3(PixelPos, 0));
#else
		float4 GBufferE = 1;
#endif
		return GBufferE;
	}
	return SubstratePixelHeader.HasZeroPrecShadowMask() ? 0.0f : 1.0f;
}

#endif // SUBSTRATE_ENABLED

////////////////////////////////////////////////////////////////////////////////////////////////////

// Affect the light color by any other shadow/transmittance before the lighting is evaluated
// in order to get a correct alpha channel containing important information for SSS.
void UpdateLightDataColor(inout float3 LightDataColor, float2 InScreenPos, float2 InScreenUV, float InDeviceZ, float3 InTranslatedWorldPosition)
{
	const float Attenuation = ComputeLightProfileMultiplier(InTranslatedWorldPosition, GetDeferredLightTranslatedWorldPosition(), -DeferredLightUniforms.Direction, DeferredLightUniforms.Tangent, DeferredLightUniforms.IESAtlasIndex);
	const float3 AttenuationRGB = Attenuation * GetAtmosphereAndCloudAttenuation(GetStableTranslatedLargeWorldPosition(InScreenPos, InDeviceZ), InScreenUV);

	LightDataColor *= AttenuationRGB;
}

void UpdateLightDataColor(inout float3 LightDataColor, FInputParams InputParams, float3 InTranslatedWorldPosition)
{
	UpdateLightDataColor(LightDataColor, InputParams.PixelPos, InputParams.ScreenUV, LookupDeviceZ(InputParams.ScreenUV), InTranslatedWorldPosition);
}

#if USE_HAIR_LIGHTING == 0

void DeferredLightPixelMain(
#if LIGHT_SOURCE_SHAPE > 0
	float4 InScreenPosition : TEXCOORD0,
#else
	float2 ScreenUV			: TEXCOORD0,
	float3 ScreenVector		: TEXCOORD1,
#endif
	float4 SVPos			: SV_POSITION,
	out float4 OutColor		: SV_Target0
#if SUBSTRATE_OUTPUT_TOON_DATA
	, out float3 OutToonData0 : SV_Target1
	, out float2 OutToonData1 : SV_Target2
#endif
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	, out float3 OutOpaqueRoughRefractionSceneColor : SV_Target1
	, out float3 OutSubSurfaceSceneColor : SV_Target2
#endif
// GBuffer format and legacy path
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && !SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED && !SUBSTRATE_OUTPUT_TOON_DATA
	, out float OutSeparatedSubsurfaceDiffuse : SV_Target1
#endif
	)
{
	const float2 PixelPos = SVPos.xy;
	OutColor = 0;
#if SUBSTRATE_OUTPUT_TOON_DATA
	OutToonData0 = float3(0.0f, 0.0f, 0.0f);
	OutToonData1 = float2(0.0f, 0.0f);
#endif
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutOpaqueRoughRefractionSceneColor = 0;
	OutSubSurfaceSceneColor = 0;
#endif

	// Convert input data (directional/local light)
	FInputParams InputParams = (FInputParams)0;
	InputParams.PixelPos		= SVPos.xy;
#if LIGHT_SOURCE_SHAPE > 0
	InputParams.ScreenPosition	= InScreenPosition;
	InputParams.ScreenUV		= InScreenPosition.xy / InScreenPosition.w * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
	InputParams.ScreenVector	= 0;
#else
	InputParams.ScreenPosition	= 0;
	InputParams.ScreenUV		= ScreenUV;
	InputParams.ScreenVector	= ScreenVector;
#endif

#if SUBSTRATE_ENABLED==1

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bMaterialIsLit = SubstratePixelHeader.ClosureCount > 0;	// This test is also enough to exclude sky pixels
	#else
	const bool bForceUnlitOrDefaultLit = SUBSTRATE_FASTPATH;
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(InputParams.ScreenUV, true, bForceUnlitOrDefaultLit);
	const FGBufferData GBuffer = ScreenSpaceData.GBuffer;
	const bool bMaterialIsLit = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
	#endif

	float EstimatedCost = 0.3; // running the PixelShader at all has a cost

	BRANCH
	if (
		bMaterialIsLit
#if USE_LIGHTING_CHANNELS
		&& (GetLightingChannelMask(InputParams.ScreenUV) & DeferredLightUniforms.LightingChannelMask)
#endif
		) 
	{
		const float SceneDepth = CalcSceneDepth(InputParams.ScreenUV);
		const FDerivedParams DerivedParams = GetDerivedParams(InputParams, SceneDepth);

		FDeferredLightData LightData = InitDeferredLightFromUniforms(CURRENT_LIGHT_TYPE);
		UpdateLightDataColor(LightData.Color, InputParams, DerivedParams.TranslatedWorldPosition);

		float3 V =-DerivedParams.CameraVector;
		float3 L = LightData.Direction;	// Already normalized
		float3 ToLight = L;
		float LightMask = 1;
		if (LightData.bRadialLight)
		{
			LightMask = GetLocalLightAttenuation(DerivedParams.TranslatedWorldPosition, LightData, ToLight, L);
		}

		if (LightMask > 0)
		{
			FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters	= GetInitialisedSubstrateShadowTermInputParameters();
			SubstrateShadowTermInputParameters.bEvaluateShadowTerm					= true;
			SubstrateShadowTermInputParameters.SceneDepth							= SceneDepth;
		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
			SubstrateShadowTermInputParameters.PrecomputedShadowFactors				= SubstrateReadPrecomputedShadowFactors(SubstratePixelHeader, PixelPos, SceneTexturesStruct.GBufferETexture);
		#else
			SubstrateShadowTermInputParameters.PrecomputedShadowFactors				= GBuffer.PrecomputedShadowFactors;
		#endif
			SubstrateShadowTermInputParameters.TranslatedWorldPosition				= DerivedParams.TranslatedWorldPosition;
			SubstrateShadowTermInputParameters.LightAttenuation						= GetLightAttenuationFromShadow(InputParams, SceneDepth, DerivedParams.TranslatedWorldPosition);
			SubstrateShadowTermInputParameters.Dither								= InterleavedGradientNoise(InputParams.PixelPos, View.StateFrameIndexMod8);
			
		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER==0

			FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);

		#endif

			FSubstrateDeferredLighting SubstrateLighting = SubstrateDeferredLighting(
				LightData,
				V,
				L,
				ToLight,
				LightMask,
				SubstrateShadowTermInputParameters,
			#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
				Substrate.MaterialTextureArray,
				SubstrateAddressing,
				SubstratePixelHeader
			#else
				PixelPos,
				GBufferBSDF.BSDF,
				GBufferBSDF.BSDFTangentBasis,
				GBufferBSDF.BSDFAO
			#endif
			);

			EstimatedCost += SubstrateLighting.EstimatedCost;

			OutColor += SubstrateLighting.SceneColor;
		#if SUBSTRATE_OUTPUT_TOON_DATA
			// We do not need to test for toon BSDF as it is a single BSDF and always will work correctly when needed.
			{
				const float3 ExposedDiffuseLuminance = SubstrateLighting.ToonData.DiffuseLuminance * GetExposure();
				const float FinalWeight = SubstrateLighting.ToonData.Weight * GetExposure(); // expose the weight which have HDR light contribution in it.

				OutToonData0 = ExposedDiffuseLuminance;
				OutToonData1 = float2(FinalWeight, FinalWeight * SubstrateLighting.ToonData.ShadowNoL);
			}
		#endif
		#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
			OutOpaqueRoughRefractionSceneColor += SubstrateLighting.OpaqueRoughRefractionSceneColor;
			OutSubSurfaceSceneColor += SubstrateLighting.SubSurfaceSceneColor;
		#endif
		}
	}

#if VISUALIZE_LIGHT_CULLING == 1
	// Convert estimated cost into a color code.
	// Legacy stored the color code into both diffuse and specular term, and added. 
	// Doing the same to match the legacy output.
	const float4 CostWithColorCode = ConvertEstimatedCostToColor(EstimatedCost);
	OutColor = CostWithColorCode + CostWithColorCode;
	#if SUBSTRATE_OUTPUT_TOON_DATA
	OutToonData0 = float3(0.0f, 0.0f, 0.0f);
	OutToonData1 = float2(0.0f, 0.0f);
	#endif
	#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutOpaqueRoughRefractionSceneColor = 0.0f;
	OutSubSurfaceSceneColor = 0.0f;
	#endif
#endif

#else // SUBSTRATE_ENABLED==1

	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(InputParams.ScreenUV);
	// Only light pixels marked as using deferred shading
	BRANCH if (ScreenSpaceData.GBuffer.ShadingModelID > 0
#if USE_LIGHTING_CHANNELS
		&& (GetLightingChannelMask(InputParams.ScreenUV) & DeferredLightUniforms.LightingChannelMask)
#endif
		)
	{
		const float SceneDepth = CalcSceneDepth(InputParams.ScreenUV);
		const FDerivedParams DerivedParams = GetDerivedParams(InputParams, SceneDepth);

		FDeferredLightData LightData = InitDeferredLightFromUniforms(CURRENT_LIGHT_TYPE);
		UpdateLightDataColor(LightData.Color, InputParams, DerivedParams.TranslatedWorldPosition);

	 #if USE_HAIR_COMPLEX_TRANSMITTANCE
		if (ScreenSpaceData.GBuffer.ShadingModelID == SHADINGMODELID_HAIR && ShouldUseHairComplexTransmittance(ScreenSpaceData.GBuffer))
		{
			LightData.HairTransmittance = EvaluateDualScattering(ScreenSpaceData.GBuffer, DerivedParams.CameraVector, -DeferredLightUniforms.Direction);
		}
	#endif

		float Dither = InterleavedGradientNoise(InputParams.PixelPos, View.StateFrameIndexMod8);

		float SurfaceShadow = 1.0f;
		
		float4 LightAttenuation = GetLightAttenuationFromShadow(InputParams, SceneDepth, DerivedParams.TranslatedWorldPosition);
		float4 Radiance = GetDynamicLighting(DerivedParams.TranslatedWorldPosition, DerivedParams.CameraVector, ScreenSpaceData.GBuffer, ScreenSpaceData.AmbientOcclusion, LightData, LightAttenuation, Dither, uint2(InputParams.PixelPos), SurfaceShadow);

		OutColor += Radiance;
	}

#endif // SUBSTRATE_ENABLED==1

	// RGB:SceneColor Specular and Diffuse
	// A:Non Specular SceneColor Luminance
	// So we need PreExposure for both color and alpha
	OutColor.rgba *= GetExposure();

#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && !SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED && !SUBSTRATE_OUTPUT_TOON_DATA
	OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif

#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED 
	// Idem
	OutOpaqueRoughRefractionSceneColor *= GetExposure();
	OutSubSurfaceSceneColor *= GetExposure();
#endif
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

#if USE_HAIR_LIGHTING == 1

// Compute transmittance data from a transmittance mask
FHairTransmittanceData GetTransmittanceDataFromTransmitttanceMask(
	FDeferredLightData LightData, 
	FGBufferData GBufferData, 
	FHairTransmittanceMask TransmittanceMask,
	float3 TranslatedWorldPosition, 
	float3 CameraVector)
{
	float3 L = float3(0, 0, 1);
	if (LightData.bRadialLight)
	{
		L = normalize(LightData.TranslatedWorldPosition - TranslatedWorldPosition);
	}
	else
	{
		L = LightData.Direction;
	}

	const float3 V = normalize(-CameraVector);
	return GetHairTransmittance(
		V,
		L,
		GBufferData,
		TransmittanceMask,
		View.HairScatteringLUTTexture,
		HairScatteringLUTSampler,
		View.HairComponents);
}

Texture2D<float4> ScreenShadowMaskSubPixelTexture;

uint HairTransmittanceBufferMaxCount;
Buffer<uint> HairTransmittanceBuffer;
uint HairShadowMaskValid;
float4 ShadowChannelMask;

void DeferredLightPixelMain(
	float4 SVPos : SV_POSITION,
	nointerpolation uint NodeCount  : DISPATCH_NODECOUNT,
	nointerpolation uint2 Resolution : DISPATCH_RESOLUTION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	const uint2 InCoord = uint2(SVPos.xy);
	const uint SampleIndex = InCoord.x + InCoord.y * Resolution.x;
	if (SampleIndex >= NodeCount)
	{
		return;
	}

	const uint2 PixelCoord = HairStrands.HairSampleCoords[SampleIndex];
	const float2 UV = (PixelCoord + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	const float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	
	const FPackedHairSample PackedSample = HairStrands.HairSampleData[SampleIndex];
	const FHairSample Sample = UnpackHairSample(PackedSample);

	#if USE_LIGHTING_CHANNELS
	if (!(Sample.LightChannelMask & DeferredLightUniforms.LightingChannelMask))
	{
		return;
	}
	#endif

	// Inject material data from the visibility/mini-gbuffer for hair
	const float SceneDepth = ConvertFromDeviceZ(Sample.Depth);
	FScreenSpaceData HairScreenSpaceData = (FScreenSpaceData)0;
	HairScreenSpaceData.AmbientOcclusion = 1;
	HairScreenSpaceData.GBuffer = HairSampleToGBufferData(Sample, HairStrands.HairDualScatteringRoughnessOverride);

	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, HairScreenSpaceData.GBuffer.Depth), HairScreenSpaceData.GBuffer.Depth, 1), View.ScreenToTranslatedWorld).xyz;
	const float3 CameraVector = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);

	FDeferredLightData LightData = InitDeferredLightFromUniforms(CURRENT_LIGHT_TYPE);
	UpdateLightDataColor(LightData.Color, ScreenPosition, UV, Sample.Depth, TranslatedWorldPosition);

	// todo add light culling/early out

	// Compute light attenuation from shadow mask, and fetch hair transmittance mask
	// * Either use VSM shadow mask bits AND hair one pass transmittance buffer
	// * Or use ShadowMask texture AND TransmittanceMask buffer
	const float Dither = InterleavedGradientNoise(PixelCoord, View.StateFrameIndexMod8);
	float4 LightAttenuation = 1.f;
	FHairTransmittanceMask TransmittanceMask = InitHairTransmittanceMask();
	#if USE_VIRTUAL_SHADOW_MAP_MASK
	if (VirtualShadowMapId != INDEX_NONE)
	{
		// Fetch VSM shadow mask
		uint GridLightListIndex = 0;
		LightAttenuation = GetVirtualShadowMapMaskForLight(
			ShadowMaskBits[PixelCoord],
			PixelCoord,
			SceneDepth,
			VirtualShadowMapId,
			TranslatedWorldPosition,
			GridLightListIndex);

		// Fetch transmittance mask
		if (GridLightListIndex < GetPackedShadowMaskMaxLightCount())
		{
			// Use VSM GridLightListIndex, which contains data only for shadowed light. 
			// The index computation needs to be identical as in HairStrandsDeepTransmittanceMask.usf
			const uint TotalSampleCount = NodeCount;
			const uint TransmittanceSampleIndex = SampleIndex + GridLightListIndex * TotalSampleCount;
			const uint TransmittanceSampleMaxCount = TotalSampleCount * GetPackedShadowMaskMaxLightCount();
			if (TransmittanceSampleIndex < TransmittanceSampleMaxCount)
			{
				TransmittanceMask = UnpackTransmittanceMask(HairTransmittanceBuffer[TransmittanceSampleIndex]);
			}
		}
	}
	#else
	if (HairShadowMaskValid)
	{
		LightAttenuation = ScreenShadowMaskSubPixelTexture.Load(uint3(PixelCoord, 0));
	}
	if (SampleIndex < HairTransmittanceBufferMaxCount)
	{
		TransmittanceMask = UnpackTransmittanceMask(HairTransmittanceBuffer[SampleIndex]);
	}
	#endif

	// Compute hair transmittance data
	{
		LightData.HairTransmittance = GetTransmittanceDataFromTransmitttanceMask(LightData, HairScreenSpaceData.GBuffer, TransmittanceMask, TranslatedWorldPosition, CameraVector);
	}

	if (any(ShadowChannelMask < 1.0f))
	{
		LightAttenuation = dot(LightAttenuation, ShadowChannelMask).xxxx;
	}
	LightAttenuation = min(LightAttenuation, LightData.HairTransmittance.OpaqueVisibility.xxxx);

#if SUBSTRATE_ENABLED==1

	float3 V =-CameraVector;
	float3 L = LightData.Direction;	// Already normalized
	float3 ToLight = L;
	float LightMask = 1;
	if (LightData.bRadialLight)
	{
		LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
	}

	if (LightMask > 0)
	{
		FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters	= GetInitialisedSubstrateShadowTermInputParameters();
		SubstrateShadowTermInputParameters.bEvaluateShadowTerm					= true;
		SubstrateShadowTermInputParameters.SceneDepth							= SceneDepth;
		SubstrateShadowTermInputParameters.PrecomputedShadowFactors 			= 1.f;
		SubstrateShadowTermInputParameters.TranslatedWorldPosition				= TranslatedWorldPosition;
		SubstrateShadowTermInputParameters.LightAttenuation 					= LightAttenuation;
		SubstrateShadowTermInputParameters.Dither								= InterleavedGradientNoise(PixelCoord, View.StateFrameIndexMod8);
		SubstrateShadowTermInputParameters.bUseShadowTermHairTransmittance 		= true;
			
		FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(HairScreenSpaceData);

		FSubstrateDeferredLighting SubstrateLighting = SubstrateDeferredLighting(
			LightData,
			V,
			L,
			ToLight,
			LightMask,
			SubstrateShadowTermInputParameters,
			PixelCoord,
			GBufferBSDF.BSDF,
			GBufferBSDF.BSDFTangentBasis,
			GBufferBSDF.BSDFAO
		);

		const float LocalCoverage = From8bitCoverage(Sample.Coverage8bit);

		OutColor.rgb = SubstrateLighting.SceneColor.xyz * LocalCoverage * GetExposure();
		OutColor.a = LocalCoverage;
	}
#else // SUBSTRATE_ENABLED

	float SurfaceShadow = 1.0f;
	const float4 Radiance = GetDynamicLighting(TranslatedWorldPosition, CameraVector, HairScreenSpaceData.GBuffer, HairScreenSpaceData.AmbientOcclusion, LightData, LightAttenuation, Dither, PixelCoord, SurfaceShadow);

	const float LocalCoverage = From8bitCoverage(Sample.Coverage8bit);

	OutColor.rgb = Radiance.xyz * LocalCoverage * GetExposure();
	OutColor.a = LocalCoverage;
#endif // SUBSTRATE_ENABLED
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef DeferredLightApplyToonDiffusePS

Texture2D<float3> ToonDataTexture0;
Texture2D<float2> ToonDataTexture1;

void DeferredLightApplyToonDiffusePS(
	in float2 ScreenUV : TEXCOORD0,
	in float3 ScreenVector : TEXCOORD1,
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
)
{
	const float2 PixelPos = SVPos.xy;
	OutColor = 0;


	FInputParams InputParams	= (FInputParams)0;
	InputParams.PixelPos		= SVPos.xy;
	InputParams.ScreenPosition	= SvPositionToScreenPosition(SVPos);
	InputParams.ScreenUV		= ScreenUV;
	InputParams.ScreenVector	= ScreenVector;

#if SUBSTRATE_ENABLED==1

#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bMaterialIsLit = SubstratePixelHeader.ClosureCount > 0;	// This test is also enough to exclude sky pixels
#else
	const bool bForceUnlitOrDefaultLit = SUBSTRATE_FASTPATH;
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(InputParams.ScreenUV, true, bForceUnlitOrDefaultLit);
	const FGBufferData GBuffer = ScreenSpaceData.GBuffer;
	const bool bMaterialIsLit = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
#endif

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	if (SubstratePixelHeader.GetComplexPathMode() == COMPLEXPATH_MODE_TOON)	// TODO turn off that test This test is in fact no necessary since toon is single BSDF and always min will work and no leak cross pixel
	{
		FSubstrateBSDF BSDF = UnpackSubstrateBSDF(MaterialBuffer, SubstrateAddressing, SubstratePixelHeader);
		float ToonProfileId = TOON_PROFILEID(BSDF);
		const float2 ToonPatternUV = TOON_PATTERNUVS(BSDF);
	#else
	if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
	{
		float ToonProfileId = 0.0f;
		float2 ToonPatternUV = 0.0;
		UnpackToonCustomData(PackRGBA8(GBuffer.CustomData), ToonPatternUV, ToonProfileId);
	#endif

		const float3 ToonExposedDiffuse	= ToonDataTexture0[SVPos.xy];
		const float2 ToonWeightNoL		= ToonDataTexture1[SVPos.xy];
		const float ToonWeight			= ToonWeightNoL.x;
		const float ToonWeightedNoL		= ToonWeightNoL.y;

		FToonProfileEvaluateSettings ToonEvalSettings = GetToonProfileEvaluateSettings(ToonProfileId);
		float ToonProfileDiffuseDither = EvaluateToonProfileDiffuseDither(ToonProfileId, ToonPatternUV, ToonEvalSettings);

		// Recoverd NoL from  (NoL * Weight) / Weight, to apply the diffuse profile, accounting for the weight many light contribution at once.
		const float NoL = (ToonWeightedNoL / max(0.0001f, ToonWeight));

		const float3 ToonExposedDiffuseWithRamp = ToonExposedDiffuse * EvaluateToonProfileDiffuseRamp(ToonProfileId, NoL + ToonProfileDiffuseDither);

		OutColor = float4(ToonExposedDiffuseWithRamp, Luminance(ToonExposedDiffuseWithRamp));
	}
	else
	{
		clip(-1);
	}

#endif

}


#endif // DeferredLightApplyToonDiffusePS
