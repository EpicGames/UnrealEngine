// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredLightVertexShaders.usf: 
=============================================================================*/

#include "HairStrands/HairStrandsVisibilityCommonStruct.ush" 
#include "Common.ush"  

#if defined(SHADER_RADIAL_LIGHT)
float4 StencilingGeometryPosAndScale;
float4 StencilingConeParameters;	// .x NumSides (0 if not cone), .y NumSlices, .z ConeAngle, .w ConeSphereRadius
float4x4 StencilingConeTransform;
#endif

#if defined(SHADER_RADIAL_LIGHT) && SHADER_RADIAL_LIGHT == 0

float4 PosScaleBias;
float4 UVScaleBias;
float4 InvTargetSizeAndTextureSize; 

void DrawFullScreenRect(
	in float4 InPosition,
	in float2 InTexCoord,
	out float4 OutPosition,
	out float2 OutTexCoord)
{
	OutPosition = InPosition;
	OutPosition.xy = -1.0f + 2.0f * (PosScaleBias.zw + (InPosition.xy * PosScaleBias.xy)) * InvTargetSizeAndTextureSize.xy;
	OutPosition.xy *= float2(1, -1);
	OutTexCoord.xy = (UVScaleBias.zw + (InTexCoord.xy * UVScaleBias.xy)) * InvTargetSizeAndTextureSize.zw;
}

/** Vertex shader for rendering a directional light using a full screen quad. */
void VertexMain(
	in float2 InPosition : ATTRIBUTE0,
	in float2 InUV       : ATTRIBUTE1,
	out float2 OutTexCoord : TEXCOORD0,
	out float3 OutScreenVector : TEXCOORD1,
	out float4 OutPosition : SV_POSITION
	)
{	
	DrawFullScreenRect(float4(InPosition.xy, 0, 1), InUV, OutPosition, OutTexCoord);
	OutScreenVector = ScreenVectorFromScreenRect(float4(OutPosition.xy, 1, 0));
}
#endif

#if defined(SHADER_RADIAL_LIGHT) && SHADER_RADIAL_LIGHT == 1
/** Vertex shader for rendering a point or spot light using approximate bounding geometry. */
void VertexMain(
	in uint InVertexId : SV_VertexID,
	in float3 InPosition : ATTRIBUTE0,
	out float4 OutScreenPosition : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	float3 TranslatedWorldPosition;
	uint NumSides = StencilingConeParameters.x;
	if (NumSides != 0)
	{
		float SphereRadius = StencilingConeParameters.w;
		float ConeAngle = StencilingConeParameters.z;

		// Cone vertex shader
		const float InvCosRadiansPerSide = 1.0f / cos(PI / (float)NumSides);
		// Use Cos(Theta) = Adjacent / Hypotenuse to solve for the distance of the end of the cone along the cone's Z axis
		const float ZRadius = SphereRadius * cos(ConeAngle);
		const float TanConeAngle = tan(ConeAngle);

		uint NumSlices = StencilingConeParameters.y;
		uint CapIndexStart = NumSides * NumSlices;
		// Generate vertices for the cone shape
		if (InVertexId < CapIndexStart)
		{
			uint SliceIndex = InVertexId / NumSides;
			uint SideIndex = InVertexId % NumSides;

			const float CurrentAngle = SideIndex * 2 * PI / (float)NumSides;
			const float DistanceDownConeDirection = ZRadius * SliceIndex / (float)(NumSlices - 1);
			// Use Tan(Theta) = Opposite / Adjacent to solve for the radius of this slice
			// Boost the effective radius so that the edges of the circle lie on the cone, instead of the vertices
			const float SliceRadius = DistanceDownConeDirection * TanConeAngle * InvCosRadiansPerSide;
			// Create a position in the local space of the cone, forming a circle in the XY plane, and offsetting along the Z axis
			const float3 LocalPosition = float3(ZRadius * SliceIndex / (float)(NumSlices - 1), SliceRadius * sin(CurrentAngle), SliceRadius * cos(CurrentAngle));
			// Transform to world space and apply pre-view translation, since these vertices will be used with a shader that has pre-view translation removed
			TranslatedWorldPosition = mul(float4(LocalPosition, 1), StencilingConeTransform).xyz;
		}
		else
		{
			// Generate vertices for the spherical cap
			const float CapRadius = ZRadius * tan(ConeAngle);

			uint VertexId = InVertexId - CapIndexStart;
			uint SliceIndex = VertexId / NumSides;
			uint SideIndex = VertexId % NumSides;

			const float UnadjustedSliceRadius = CapRadius * SliceIndex / (float)(NumSlices - 1);
			// Boost the effective radius so that the edges of the circle lie on the cone, instead of the vertices
			const float SliceRadius = UnadjustedSliceRadius * InvCosRadiansPerSide;
			// Solve for the Z axis distance that this slice should be at using the Pythagorean theorem
			const float ZDistance = sqrt(SphereRadius * SphereRadius - UnadjustedSliceRadius * UnadjustedSliceRadius);

			const float CurrentAngle = SideIndex * 2 * PI / (float)NumSides;
			const float3 LocalPosition = float3(ZDistance, SliceRadius * sin(CurrentAngle), SliceRadius * cos(CurrentAngle));
			TranslatedWorldPosition = mul(float4(LocalPosition, 1), StencilingConeTransform).xyz;
		}
	}
	else
	{
		// Sphere
		TranslatedWorldPosition = InPosition * StencilingGeometryPosAndScale.w + StencilingGeometryPosAndScale.xyz;
	}

	OutScreenPosition = OutPosition = mul(float4(TranslatedWorldPosition, 1), View.TranslatedWorldToClip);
}
#endif

#if defined(SHADER_HAIR) && SHADER_HAIR == 1

#include "HairStrands/HairStrandsVisibilityCommon.ush"

void HairVertexMain(
	in uint InVertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	nointerpolation out uint OutNodeCount : DISPATCH_NODECOUNT,
	nointerpolation out uint2 OutResolution : DISPATCH_RESOLUTION)
{
	OutNodeCount = HairStrands.HairSampleCount[0];
	OutResolution = GetHairSampleResolution(OutNodeCount);

	const float2 ClipCoord = ((OutResolution + 0.5f) / float2(HairStrands.HairSampleViewportResolution)) * 2;

	const float2 UV = float2(InVertexId & 1, InVertexId >> 1);
	const float2 Pos = float2(-UV.x, UV.y) * 4 + float2(-1, +1) + float2(ClipCoord.x, -ClipCoord.y);
	OutPosition = float4(Pos, 0.5f, 1);
}

#endif
