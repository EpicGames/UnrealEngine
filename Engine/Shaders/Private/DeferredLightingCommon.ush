// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredLightingCommon.usf: Common definitions for deferred lighting.
=============================================================================*/

#pragma once

#include "DeferredShadingCommon.ush"
#include "DynamicLightingCommon.ush"
#include "IESLightProfilesCommon.ush"
#include "CapsuleLightIntegrate.ush"
#include "RectLightIntegrate.ush"
#include "ScreenSpaceShadowRayCast.ush"
#include "Substrate/Substrate.ush"
#include "LightData.ush"

#if USE_LIGHT_FUNCTION_ATLAS
#include "LightFunctionAtlas/LightFunctionAtlasCommon.usf"
#endif

// This Substrate include assumes that if inline shading is required, it needs to 
// be defined prior to DeferredLightingCommong.ush
// SubstrateStruct.MaterialTextureArray is only available to translucent materials or global shaders computing contact shadows.
// It is not defined for opaque materials, only the UAV is available to write the Substrate buffer.
#define SUBSTRATE_RAYCAST_ENABLED (!MATERIAL_IS_SUBSTRATE || SUBSTRATE_TRANSLUCENT_ENABLED)
#if SUBSTRATE_RAYCAST_ENABLED
	#if MATERIAL_IS_SUBSTRATE
		#define SubstrateRayCast SubstrateStruct
	#else
		#define SubstrateRayCast Substrate
	#endif
#endif

#define REFERENCE_QUALITY	0

/** Returns 0 for positions closer than the fade near distance from the camera, and 1 for positions further than the fade far distance. */
float DistanceFromCameraFade(float SceneDepth, FDeferredLightData LightData)
{
	// depth (non radial) based fading over distance
	float Fade = saturate(SceneDepth * LightData.DistanceFadeMAD.x + LightData.DistanceFadeMAD.y);
	return Fade * Fade;
}

#if SUPPORT_CONTACT_SHADOWS
// Returns distance along ray that the first hit occurred, or negative on miss
// Sets bOutHitCastDynamicShadow if the hit point is marked as a dynamic shadow caster
float ShadowRayCast(
	float3 RayOriginTranslatedWorld, float3 RayDirection, float RayLength,
	int NumSteps, float Dither, bool bHairNoShadowLight, out bool bOutHitCastContactShadow)
{
	// *2 to get less moire pattern in extreme cases, larger values make object appear not grounded in reflections
	const float CompareToleranceScale = 2.0f;

	float2 HitUV;
	float HitDistance = CastScreenSpaceShadowRay(RayOriginTranslatedWorld, RayDirection, RayLength, NumSteps, Dither, CompareToleranceScale, bHairNoShadowLight, HitUV);

	bOutHitCastContactShadow = false;
	if (HitDistance > 0.0)
	{
#if SHADING_PATH_MOBILE
		const uint2 HitPixelCoord = uint2(HitUV * View.BufferSizeAndInvSize.xy);
		FGBufferData SampleGBuffer = MobileFetchAndDecodeGBuffer(HitUV, HitPixelCoord);
		bOutHitCastContactShadow = CastContactShadow(SampleGBuffer);
#else
#if SUBSTRATE_GBUFFER_FORMAT==1
#if SUBSTRATE_RAYCAST_ENABLED && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
		uint2 PixelPos = View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy * HitUV;
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), SubstrateRayCast.MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(SubstrateRayCast.MaterialTextureArray, SubstrateAddressing, SubstrateRayCast.TopLayerTexture);
		// We check if the hit pixel has CastContactShadow, and we always exclude EYE BSDFs from there as done by CastContactShadow(SampleGBuffer) in the legacy path below.
		// SUBSTRATE_TODO: try to move that test to GBuffer export pass (set DoesCastContactShadow = false for EYE BSDFs).
		bOutHitCastContactShadow = SubstratePixelHeader.DoesCastContactShadow() && SubstratePixelHeader.SubstrateGetBSDFType() != SUBSTRATE_BSDF_TYPE_EYE;
#endif
#else
		FGBufferData SampleGBuffer = GetGBufferData(HitUV);
		bOutHitCastContactShadow = CastContactShadow(SampleGBuffer);
#endif
#endif
	}

	return HitDistance;
}
#endif

#ifndef SUPPORT_CONTACT_SHADOWS
#error "Must set SUPPORT_CONTACT_SHADOWS"
#endif

void GetShadowTermsBase(
	float SceneDepth, 
	half4 PrecomputedShadowFactors, 
	FDeferredLightData LightData, 
	half4 LightAttenuation, 
	inout FShadowTerms OutShadow)
{
	BRANCH
	if (LightData.ShadowedBits)
	{
		// Remapping the light attenuation buffer (see ShadowRendering.cpp)
		//
		// LightAttenuation is decoded unprocessed ShadowMask for this pixel: (GetPerPixelLightAttenuation from GetLightAttenuationFromShadow) 
		// X - Whole scene directional light shadows
		// Y - Whole scene directional light SSS shadows
		// Z - Light function + per-object shadows
		// W - per-object SSS shadows
		//
		// Get static shadowing from the appropriate GBuffer channel
#if ALLOW_STATIC_LIGHTING
		half UsesStaticShadowMap = dot(LightData.ShadowMapChannelMask, half4(1, 1, 1, 1));
		half StaticShadowing = lerp(1, dot(PrecomputedShadowFactors, LightData.ShadowMapChannelMask), UsesStaticShadowMap);
#else
		half StaticShadowing = 1.0f;
#endif

		if (LightData.bRadialLight || SHADING_PATH_MOBILE)
		{
			// Remapping the light attenuation buffer (see ShadowRendering.cpp)

			OutShadow.SurfaceShadow = LightAttenuation.z * StaticShadowing;
			// SSS uses a separate shadowing term that allows light to penetrate the surface
			//@todo - how to do static shadowing of SSS correctly?
			OutShadow.TransmissionShadow = LightAttenuation.w * StaticShadowing;

			OutShadow.TransmittanceOrOpticalThickness = LightAttenuation.w;
		}
		else
		{
			// Remapping the light attenuation buffer (see ShadowRendering.cpp)
			// Also fix up the fade between dynamic and static shadows
			// to work with plane splits rather than spheres.

			float DynamicShadowFraction = DistanceFromCameraFade(SceneDepth, LightData);
			// For a directional light, fade between static shadowing and the whole scene dynamic shadowing based on distance + per object shadows
			OutShadow.SurfaceShadow = lerp(LightAttenuation.x, StaticShadowing, DynamicShadowFraction);
			// Fade between SSS dynamic shadowing and static shadowing based on distance
			OutShadow.TransmissionShadow = min(lerp(LightAttenuation.y, StaticShadowing, DynamicShadowFraction), LightAttenuation.w);

			OutShadow.SurfaceShadow *= LightAttenuation.z;
			OutShadow.TransmissionShadow *= LightAttenuation.z;

			// Need this min or backscattering will leak when in shadow which cast by non perobject shadow(Only for directional light)
			OutShadow.TransmittanceOrOpticalThickness = min(LightAttenuation.y, LightAttenuation.w);
		}
	}

	OutShadow.HairTransmittance = LightData.HairTransmittance;
	OutShadow.HairTransmittance.OpaqueVisibility = OutShadow.SurfaceShadow;
}

void ApplyContactShadowWithShadowTerms(
	float SceneDepth, 
	uint ShadingModelID, 
	float ContactShadowOpacity, 
	FDeferredLightData LightData, 
	float3 TranslatedWorldPosition, 
	half3 L, 
	float Dither, 
	inout FShadowTerms OutShadow)
{
#if SUPPORT_CONTACT_SHADOWS

	float ContactShadowLength = 0.0f;
	const float ContactShadowLengthScreenScale = GetScreenRayLengthMultiplierForProjectionType(SceneDepth).y;

	FLATTEN
	if (LightData.ShadowedBits > 1 && LightData.ContactShadowLength > 0)
	{
		ContactShadowLength = LightData.ContactShadowLength * (LightData.ContactShadowLengthInWS ? 1.0f : ContactShadowLengthScreenScale);
	}

	if (LightData.ShadowedBits < 2 && (ShadingModelID == SHADINGMODELID_HAIR))
	{
		ContactShadowLength = 0.2 * ContactShadowLengthScreenScale;
	}
	// World space distance to cover eyelids and eyelashes but not beyond
	if (ShadingModelID == SHADINGMODELID_EYE)
	{
		ContactShadowLength = 0.5;
		
	}

#if MATERIAL_CONTACT_SHADOWS
	ContactShadowLength = 0.2 * ContactShadowLengthScreenScale;
#endif

	BRANCH
	if (ContactShadowLength > 0.0)
	{
		bool bHitCastContactShadow = false;
		bool bHairNoShadowLight = ShadingModelID == SHADINGMODELID_HAIR && !LightData.ShadowedBits;
		float HitDistance = ShadowRayCast( TranslatedWorldPosition, L, ContactShadowLength, 8, Dither, bHairNoShadowLight, bHitCastContactShadow );
				
		if ( HitDistance > 0.0 )
		{
			float ContactShadowOcclusion = bHitCastContactShadow ? LightData.ContactShadowCastingIntensity : LightData.ContactShadowNonCastingIntensity;

			// Exponential attenuation is not applied on hair/eye/SSS-profile here, as the hit distance (shading-point to blocker) is different from the estimated 
			// thickness (closest-point-from-light to shading-point), and this creates light leaks. Instead we consider first hit as a blocker (old behavior)
			BRANCH
			if (ContactShadowOcclusion > 0.0 && 
				IsSubsurfaceModel(ShadingModelID) &&
				ShadingModelID != SHADINGMODELID_HAIR &&
				ShadingModelID != SHADINGMODELID_EYE &&
				ShadingModelID != SHADINGMODELID_SUBSURFACE_PROFILE)
			{
				// Reduce the intensity of the shadow similar to the subsurface approximation used by the shadow maps path
				// Note that this is imperfect as we don't really have the "nearest occluder to the light", but this should at least
				// ensure that we don't darken-out the subsurface term with the contact shadows
				float Density = SubsurfaceDensityFromOpacity(ContactShadowOpacity);
				ContactShadowOcclusion *= 1.0 - saturate( exp( -Density * HitDistance ) );
			}

			float ContactShadow = 1.0 - ContactShadowOcclusion;

			OutShadow.SurfaceShadow *= ContactShadow;
			OutShadow.TransmissionShadow *= ContactShadow;
		}
	}

	OutShadow.HairTransmittance = LightData.HairTransmittance;
	OutShadow.HairTransmittance.OpaqueVisibility = OutShadow.SurfaceShadow;

#endif // SUPPORT_CONTACT_SHADOWS
}

void GetShadowTerms(
	float SceneDepth,
	half4 PrecomputedShadowFactors,
	uint ShadingModelID,
	float ContactShadowOpacity,
	FDeferredLightData LightData,
	float3 TranslatedWorldPosition,
	half3 L,
	half4 LightAttenuation,
	float Dither,
	inout FShadowTerms OutShadow)
{
	// Get the basic shadow terms
	GetShadowTermsBase(SceneDepth, PrecomputedShadowFactors, LightData, LightAttenuation, OutShadow);

	// Now combined with screens space contact shadow if necessary.
	ApplyContactShadowWithShadowTerms (SceneDepth, ShadingModelID, ContactShadowOpacity, LightData, TranslatedWorldPosition, L, Dither, OutShadow);
}

float GetLocalLightAttenuation(
	float3 TranslatedWorldPosition, 
	FDeferredLightData LightData, 
	inout float3 ToLight, 
	inout float3 L)
{
	ToLight = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
		
	float DistanceSqr = dot( ToLight, ToLight );
	L = ToLight * rsqrt( DistanceSqr );

	float LightMask;
	if (LightData.bInverseSquared)
	{
		LightMask = Square( saturate( 1 - Square( DistanceSqr * Square(LightData.InvRadius) ) ) );
		// This extra attenuation has been added for supporting existing translucent shading model and merged local lights on mobile
		#if SHADING_PATH_MOBILE && TRANSLUCENCY_SH_LIGHTING
		if (!LightData.bRectLight)
		{ 
			LightMask *= 1.0f / (DistanceSqr + 1.0f);
		}
		#endif
	}
	else
	{
		LightMask = RadialAttenuation(ToLight * LightData.InvRadius, LightData.FalloffExponent);
	}

	if (LightData.bSpotLight)
	{
		LightMask *= SpotAttenuation(L, -LightData.Direction, LightData.SpotAngles);
	}

	if( LightData.bRectLight )
	{
		// Rect normal points away from point
		LightMask = dot( LightData.Direction, L ) < 0 ? 0 : LightMask;
	}

	return LightMask;
}

#define RECLIGHT_BARNDOOR 1
// Wrapper for FDeferredLightData for computing visible rect light (i.e., unoccluded by barn doors)
FRect GetRect(float3 ToLight, FDeferredLightData LightData)
{
	return GetRect(
		ToLight, 
		LightData.Direction, 
		LightData.Tangent, 
		LightData.SourceRadius, 
		LightData.SourceLength, 
		LightData.RectLightData.BarnCosAngle, 
		LightData.RectLightData.BarnLength,
		RECLIGHT_BARNDOOR);
}

FCapsuleLight GetCapsule( float3 ToLight, FDeferredLightData LightData )
{
	FCapsuleLight Capsule;
	Capsule.Length = LightData.SourceLength;
	Capsule.Radius = LightData.SourceRadius;
	Capsule.SoftRadius = LightData.SoftSourceRadius;
	Capsule.DistBiasSqr = 1;
	Capsule.LightPos[0] = ToLight - 0.5 * Capsule.Length * LightData.Tangent;
	Capsule.LightPos[1] = ToLight + 0.5 * Capsule.Length * LightData.Tangent;
	return Capsule;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate - Uses SubstrateDeferredLighting() instead
FLightAccumulator AccumulateDynamicLighting(
	float3 TranslatedWorldPosition, half3 CameraVector, FGBufferData GBuffer, half AmbientOcclusion,
	FDeferredLightData LightData, half4 LightAttenuation, float Dither, uint2 SVPos, 
	inout float SurfaceShadow)
{
	FLightAccumulator LightAccumulator = (FLightAccumulator)0;

	half3 V = -CameraVector;
	half3 N = GBuffer.WorldNormal;
	BRANCH if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
	{
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0/255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
		N = OctahedronToUnitVector(oct1);			
	}
	
	float3 L = LightData.Direction;	// Already normalized
	float3 ToLight = L;
	float3 MaskedLightColor = LightData.Color;
	float LightMask = 1;
	if (LightData.bRadialLight)
	{
		LightMask = GetLocalLightAttenuation( TranslatedWorldPosition, LightData, ToLight, L );
		MaskedLightColor *= LightMask;
	}

	LightAccumulator.EstimatedCost += 0.3f;		// running the PixelShader at all has a cost

	BRANCH
	if( LightMask > 0 )
	{
		FShadowTerms Shadow;
		Shadow.SurfaceShadow = AmbientOcclusion;
		Shadow.TransmissionShadow = 1;
		Shadow.TransmittanceOrOpticalThickness = 1;
		Shadow.HairTransmittance.OpaqueVisibility = 1;
		const float ContactShadowOpacity = GBuffer.CustomData.a;
		GetShadowTerms(GBuffer.Depth, GBuffer.PrecomputedShadowFactors, GBuffer.ShadingModelID, ContactShadowOpacity,
			LightData, TranslatedWorldPosition, L, LightAttenuation, Dither, Shadow);
		SurfaceShadow = Shadow.SurfaceShadow;

		LightAccumulator.EstimatedCost += 0.3f;		// add the cost of getting the shadow terms

	#if !COMPILER_FXC // Work around "error X3524: duplicate attribute branch" in cases where this branch is constant
		BRANCH
	#endif
		if( Shadow.SurfaceShadow + Shadow.TransmissionShadow > 0 )
		{
			const bool bNeedsSeparateSubsurfaceLightAccumulation = UseSubsurfaceProfile(GBuffer.ShadingModelID);

		#if NON_DIRECTIONAL_DIRECT_LIGHTING
			float Lighting;

			if( LightData.bRectLight )
			{
				FRect Rect = GetRect( ToLight, LightData );

				Lighting = IntegrateLight( Rect );
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule( ToLight, LightData );

				Lighting = IntegrateLight( Capsule, LightData.bInverseSquared );
			}

			float3 LightingDiffuse = Diffuse_Lambert( GBuffer.DiffuseColor ) * Lighting;
			LightAccumulator_AddSplit(LightAccumulator, LightingDiffuse, 0.0f, 0, MaskedLightColor * Shadow.SurfaceShadow, bNeedsSeparateSubsurfaceLightAccumulation);
		#else
			FDirectLighting Lighting;

			if (LightData.bRectLight)
			{
				FRect Rect = GetRect( ToLight, LightData );
				const FRectTexture SourceTexture = ConvertToRectTexture(LightData);

				#if REFERENCE_QUALITY
					Lighting = IntegrateBxDF( GBuffer, N, V, Rect, Shadow, SourceTexture, SVPos );
				#else
					Lighting = IntegrateBxDF( GBuffer, N, V, Rect, Shadow, SourceTexture);
				#endif
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule( ToLight, LightData );

				#if REFERENCE_QUALITY
					Lighting = IntegrateBxDF( GBuffer, N, V, Capsule, Shadow, SVPos );
				#else
					Lighting = IntegrateBxDF( GBuffer, N, V, Capsule, Shadow, LightData.bInverseSquared );
				#endif
			}

			Lighting.Specular *= LightData.SpecularScale;
			Lighting.Diffuse  *= LightData.DiffuseScale;
			Lighting.Transmission *= LightData.DiffuseScale;

		#if USE_LIGHT_FUNCTION_ATLAS
			MaskedLightColor *= GetLocalLightFunctionCommon(TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
		#endif

			LightAccumulator_AddSplit( LightAccumulator, Lighting.Diffuse, Lighting.Specular, Lighting.Diffuse, MaskedLightColor * Shadow.SurfaceShadow, bNeedsSeparateSubsurfaceLightAccumulation );
			
			// SubsurfaceProfileBxDF can return non-zero transmission lighting even if this is a specular only pixel so we need to skip tranmission explicitly
			bool bOutputSpecularOnly = false;
		#if USE_TRANSMISSION && ALLOW_SSS_MATERIAL_OVERRIDE && ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE
			if (View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0 && bNeedsSeparateSubsurfaceLightAccumulation)
			{
				bOutputSpecularOnly = !CheckerFromPixelPos(SVPos);
			}
		#endif

			if (!bOutputSpecularOnly)
			{
				LightAccumulator_AddSplit( LightAccumulator, Lighting.Transmission, 0.0f, Lighting.Transmission, MaskedLightColor * Shadow.TransmissionShadow, bNeedsSeparateSubsurfaceLightAccumulation );
			}

			LightAccumulator.EstimatedCost += 0.4f;		// add the cost of the lighting computations (should sum up to 1 form one light)
		#endif
		}
	}
	return LightAccumulator;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate - Uses SubstrateDeferredLighting() instead
/** Calculates lighting for a given position, normal, etc with a fully featured lighting model designed for quality. */
FDeferredLightingSplit GetDynamicLightingSplit(
	float3 TranslatedWorldPosition, float3 CameraVector, FGBufferData GBuffer, float AmbientOcclusion, 
	FDeferredLightData LightData, float4 LightAttenuation, float Dither, uint2 SVPos, 
	inout float SurfaceShadow)
{
	FLightAccumulator LightAccumulator = AccumulateDynamicLighting(TranslatedWorldPosition, CameraVector, GBuffer, AmbientOcclusion, LightData, LightAttenuation, Dither, SVPos, SurfaceShadow);
	return LightAccumulator_GetResultSplit(LightAccumulator);
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate - Uses SubstrateDeferredLighting() instead
float4 GetDynamicLighting(
	float3 TranslatedWorldPosition, float3 CameraVector, FGBufferData GBuffer, float AmbientOcclusion, 
	FDeferredLightData LightData, float4 LightAttenuation, float Dither, uint2 SVPos, 
	inout float SurfaceShadow)
{
	FDeferredLightingSplit SplitLighting = GetDynamicLightingSplit(
		TranslatedWorldPosition, CameraVector, GBuffer, AmbientOcclusion, 
		LightData, LightAttenuation, Dither, SVPos, 
		SurfaceShadow);

	return SplitLighting.SpecularLighting + SplitLighting.DiffuseLighting;
}

/** 
 * Calculates lighting for a given position, normal, etc with a simple lighting model designed for speed. 
 * All lights rendered through this method are unshadowed point lights with no shadowing or light function or IES.
 * A cheap specular is used instead of the more correct area specular, no fresnel.
 */
float3 GetSimpleDynamicLighting(float3 TranslatedWorldPosition, float3 CameraVector, float3 WorldNormal, float AmbientOcclusion, float3 DiffuseColor, float3 SpecularColor, float Roughness, FSimpleDeferredLightData LightData)
{
	float3 V = -CameraVector;
	float3 N = WorldNormal;
	float3 ToLight = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
	float DistanceAttenuation = 1;
	
	float DistanceSqr = dot( ToLight, ToLight );
	float3 L = ToLight * rsqrt( DistanceSqr );
	float NoL = saturate( dot( N, L ) );

	if (LightData.bInverseSquared)
	{
		// Sphere falloff (technically just 1/d2 but this avoids inf)
		DistanceAttenuation = 1 / ( DistanceSqr + 1 );
	
		float LightRadiusMask = Square( saturate( 1 - Square( DistanceSqr * Square(LightData.InvRadius) ) ) );
		DistanceAttenuation *= LightRadiusMask;
	}
	else
	{
		DistanceAttenuation = RadialAttenuation(ToLight * LightData.InvRadius, LightData.FalloffExponent);
	}

	float3 OutLighting = 0;

	BRANCH
	if (DistanceAttenuation > 0)
	{
		const float3 LightColor = LightData.Color;

		// Apply SSAO to the direct lighting since we're not going to have any other shadowing
		float Attenuation = DistanceAttenuation * AmbientOcclusion;

		#if NON_DIRECTIONAL_DIRECT_LIGHTING
			float3 VolumeLighting = Diffuse_Lambert(DiffuseColor);
			OutLighting += LightColor * Attenuation * VolumeLighting;
		#else
			OutLighting += LightColor * (NoL * Attenuation) * SimpleShading(DiffuseColor, SpecularColor, max(Roughness, .04f), L, V, N);
		#endif
	}

	return OutLighting;
}

float3 GetIrradianceForLight(FDeferredLightData LightData, float3 WorldNormal,	float3 TranslatedWorldPosition, bool bSupportRectLight)
{
	float3 LightColor = LightData.Color;
	float3 L = LightData.Direction;
	float3 ToLight = L;
	float3 AreaLightFalloffColor = 1;
	float CombinedAttenuation = 1;
	float NoL = saturate(dot(WorldNormal, L));

	if (LightData.bRadialLight)
	{
		FAreaLight AreaLight = InitAreaLight();
		float LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

		float Attenuation = 0.0f;
		float Roughness = 1;
		float3 V = float3(1, 0, 0);

		if (bSupportRectLight && LightData.bRectLight)
		{
			FRect Rect = GetRect(ToLight, LightData);
			Attenuation = IntegrateLight(Rect);

			if (IsRectVisible(Rect))
			{
				const FRectTexture SourceTexture = ConvertToRectTexture(LightData);
				AreaLight = CreateAreaLight(Roughness, WorldNormal, V, Rect, SourceTexture);
			}
		}
		else
		{
			FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
			Capsule.DistBiasSqr = 0;
			AreaLight = CreateAreaLight(Roughness, WorldNormal, V, Capsule, LightData.bInverseSquared);
			Attenuation = IntegrateLight(Capsule, LightData.bInverseSquared);
		}

		CombinedAttenuation = Attenuation * LightMask;
		AreaLightFalloffColor = AreaLight.FalloffColor;
		NoL = AreaLight.NoL;
	}

	return LightColor * AreaLightFalloffColor * (CombinedAttenuation * NoL * LightData.DiffuseScale);
}

float GetSpotLightVolumetricSoftFading(in const FDeferredLightData LightData, float LightVolumetricSoftFadeDistance, in float3 ToLight)
{
	const float CosOuterCone = LightData.SpotAngles.x;
	if (CosOuterCone > 0)
	{
		const float SinOuterCone = sqrt(1 - CosOuterCone * CosOuterCone);
		const float TanOuterCone = SinOuterCone / CosOuterCone;

		const float DistanceAlongSpotDir = dot(LightData.Direction, ToLight);
		const float ApertureAtProjectedDistance = DistanceAlongSpotDir * TanOuterCone;
		const float DistanceFromProjected = length(ToLight - LightData.Direction * DistanceAlongSpotDir);

		const float Diff = ApertureAtProjectedDistance - DistanceFromProjected;
		if (Diff < LightVolumetricSoftFadeDistance)
		{
			return saturate(Diff / LightVolumetricSoftFadeDistance);
		}
	}
	return 1.0f;
}

float GetRectLightVolumetricSoftFading(in const FDeferredLightData LightData, in FRect Rect, in float LightVolumetricSoftFadeDistance, in float3 ToLight)
{
	const float LightVolumetricSoftFadeDistanceInv = 1.0f / LightVolumetricSoftFadeDistance;
	float FinalFadeOut = 1.0f;

	// Bias outward the rect light polygon to reduce alising if the voxel intersects the area light
	const float DistanceToPlane = dot(LightData.Direction, ToLight);
	FinalFadeOut *= saturate(DistanceToPlane * LightVolumetricSoftFadeDistanceInv);

	float3 LocalToLight = mul(Rect.Axis, ToLight);

	const float BarnLength = LightData.RectLightData.BarnLength;
	const float CosTheta = LightData.RectLightData.BarnCosAngle;
	const float SinTheta = sqrt(1.0f - CosTheta * CosTheta);
	const float BarnLengthOrtho = BarnLength * CosTheta;

	const float FadeFroxelCount = 4.0f;
	if (DistanceToPlane < (BarnLengthOrtho + FadeFroxelCount * LightVolumetricSoftFadeDistance))
	{
		// Fade out progressively at barn door tip to avoid harsh transition
		float BarnDoorTipFadeOut = saturate((DistanceToPlane - BarnLengthOrtho) / (FadeFroxelCount * LightVolumetricSoftFadeDistance));
		// Fade out as barn door length get shorter
		BarnDoorTipFadeOut = lerp(BarnDoorTipFadeOut, 1.0f, 1.0f - saturate(BarnLength * LightVolumetricSoftFadeDistanceInv));

		const float2 Extent = float2(LightData.SourceRadius, LightData.SourceLength);
		float BarnDoorFadeOut = 1.0f;
		{
			float3 PlaneO = float3(Extent.x, 0.0f, 0.0f);
			float3 PlaneN = float3(-CosTheta, 0.0f, SinTheta);
			float4 PlanePosX = float4(PlaneN, -dot(PlaneO, PlaneN));

			float d = dot(PlanePosX, float4(LocalToLight, 1.0f));
			BarnDoorFadeOut *= saturate(d * LightVolumetricSoftFadeDistanceInv);
		}

		{
			float3 PlaneO = float3(-Extent.x, 0.0f, 0.0f);
			float3 PlaneN = float3(CosTheta, 0.0f, SinTheta);
			float4 PlaneNegX = float4(PlaneN, -dot(PlaneO, PlaneN));

			float d = dot(PlaneNegX, float4(LocalToLight, 1.0f));
			BarnDoorFadeOut *= saturate(d * LightVolumetricSoftFadeDistanceInv);
		}

		{
			float3 PlaneO = float3(0.0f, Extent.y, 0.0f);
			float3 PlaneN = float3(0.0f, -CosTheta, SinTheta);
			float4 PlanePosY = float4(PlaneN, -dot(PlaneO, PlaneN));

			float d = dot(PlanePosY, float4(LocalToLight, 1.0f));
			BarnDoorFadeOut *= saturate(d * LightVolumetricSoftFadeDistanceInv);
		}

		{
			float3 PlaneO = float3(0.0f, -Extent.y, 0.0f);
			float3 PlaneN = float3(0.0f, CosTheta, SinTheta);
			float4 PlaneNegY = float4(PlaneN, -dot(PlaneO, PlaneN));

			float d = dot(PlaneNegY, float4(LocalToLight, 1.0f));
			BarnDoorFadeOut *= saturate(d * LightVolumetricSoftFadeDistanceInv);
		}

		FinalFadeOut *= lerp(BarnDoorFadeOut, 1.0f, BarnDoorTipFadeOut);
	}

	return FinalFadeOut;
}
