// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredShadingCommon.usf: Common definitions for deferred shading.
=============================================================================*/

#pragma once

#include "ShadingCommon.ush"
#include "LightAccumulator.ush"
#include "SceneTexturesCommon.ush"
#include "MonteCarlo.ush"
#include "OctahedralCommon.ush"

// if this is zero, use the new autogenerated GBuffer encode/decode functions
#ifndef GBUFFER_REFACTOR
#define GBUFFER_REFACTOR 0
#endif

#ifndef ALLOW_SSS_MATERIAL_OVERRIDE
#define ALLOW_SSS_MATERIAL_OVERRIDE 1
#endif

#ifndef ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE
#define ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE 1
#endif

// Whether to use only unlit or default lit shading model
#ifndef FORCE_SIMPLE_SHADING
	#define FORCE_SIMPLE_SHADING 0
#endif

#if SHADING_PATH_DEFERRED

#if !SUPPORTS_INDEPENDENT_SAMPLERS
	#error Scene textures may only be used on platforms which support independent samplers.
#endif

uint bSceneLightingChannelsValid;

// Matches FSceneTextureParameters
Texture2D SceneDepthTexture;
Texture2D<uint2> SceneStencilTexture;
Texture2D GBufferATexture;
Texture2D GBufferBTexture;
Texture2D GBufferCTexture;
Texture2D GBufferDTexture;
Texture2D GBufferETexture;
Texture2D GBufferVelocityTexture;
Texture2D GBufferFTexture;
Texture2D GBufferSGGXTexture;
Texture2D<uint> SceneLightingChannels;

#define SceneDepthTextureSampler GlobalPointClampedSampler
#define GBufferATextureSampler GlobalPointClampedSampler
#define GBufferBTextureSampler GlobalPointClampedSampler
#define GBufferCTextureSampler GlobalPointClampedSampler
#define GBufferDTextureSampler GlobalPointClampedSampler
#define GBufferETextureSampler GlobalPointClampedSampler
#define GBufferFTextureSampler GlobalPointClampedSampler
#define GBufferVelocityTextureSampler GlobalPointClampedSampler
#define GBufferSGGXTextureSampler GlobalPointClampedSampler

float SampleDeviceZFromSceneTextures(float2 UV)
{
	return SceneDepthTexture.SampleLevel(SceneDepthTextureSampler, UV, 0).r;
}

#endif

// TODO: for CustomGBufferResolvePS() MSAA_SAMPLE_COUNT is defined by C++ code as 2 or 4
// bot not for any other shaders!
#ifndef MSAA_SAMPLE_COUNT
	#define MSAA_SAMPLE_COUNT 2
#endif

float3 RGBToYCoCg( float3 RGB )
{
	float Y  = dot( RGB, float3(  1, 2,  1 ) ) * 0.25;
	float Co = dot( RGB, float3(  2, 0, -2 ) ) * 0.25 + ( 0.5 * 256.0 / 255.0 );
	float Cg = dot( RGB, float3( -1, 2, -1 ) ) * 0.25 + ( 0.5 * 256.0 / 255.0 );
	
	float3 YCoCg = float3( Y, Co, Cg );
	return YCoCg;
}

float3 YCoCgToRGB( float3 YCoCg )
{
	float Y  = YCoCg.x;
	float Co = YCoCg.y - ( 0.5 * 256.0 / 255.0 );
	float Cg = YCoCg.z - ( 0.5 * 256.0 / 255.0 );

	float R = Y + Co - Cg;
	float G = Y + Cg;
	float B = Y - Co - Cg;

	float3 RGB = float3( R, G, B );
	return RGB;
}

float3 Pack1212To888( float2 x )
{
	// Pack 12:12 to 8:8:8
#if 0
	uint2 x1212 = (uint2)( x * 4095.0 );
	uint2 High = x1212 >> 8;
	uint2 Low = x1212 & 255;
	uint3 x888 = uint3( Low, High.x | (High.y << 4) );
	return x888 / 255.0;
#else
	float2 x1212 = floor( x * 4095 );
	float2 High = floor( x1212 / 256 );	// x1212 >> 8
	float2 Low = x1212 - High * 256;	// x1212 & 255
	float3 x888 = float3( Low, High.x + High.y * 16 );
	return saturate( x888 / 255 );
#endif
}

float2 Pack888To1212( float3 x )
{
	// Pack 8:8:8 to 12:12
#if 0
	uint3 x888 = (uint3)( x * 255.0 );
	uint High = x888.z >> 4;
	uint Low = x888.z & 15;
	uint2 x1212 = x888.xy | uint2( Low << 8, High << 8 );
	return x1212 / 4095.0;
#else
	float3 x888 = floor( x * 255 );
	float High = floor( x888.z / 16 );	// x888.z >> 4
	float Low = x888.z - High * 16;		// x888.z & 15
	float2 x1212 = x888.xy + float2( Low, High ) * 256;
	return saturate( x1212 / 4095 );
#endif
}

float3 EncodeNormal( float3 N )
{
	return N * 0.5 + 0.5;
	//return Pack1212To888( UnitVectorToOctahedron( N ) * 0.5 + 0.5 );
}

float3 DecodeNormal( float3 N )
{
	return N * 2 - 1;
	//return OctahedronToUnitVector( Pack888To1212( N ) * 2 - 1 );
}

void EncodeNormal( inout float3 N, out uint Face )
{
#if 1
	uint Axis = 2;
	if( abs(N.x) >= abs(N.y) && abs(N.x) >= abs(N.z) )
	{
		Axis = 0;
	}
	else if( abs(N.y) > abs(N.z) )
	{
		Axis = 1;
	}
	Face = Axis * 2;
#else
	// TODO GCN
	Face = v_cubeid_f32( N );
	uint Axis = Face >> 1;
#endif

	N = Axis == 0 ? N.yzx : N;
	N = Axis == 1 ? N.xzy : N;
	
	float MaxAbs = 1.0 / sqrt(2.0);

	Face += N.z > 0 ? 0 : 1;
	N.xy *= N.z > 0 ? 1 : -1;
	N.xy = N.xy * (0.5 / MaxAbs) + 0.5;
}

void DecodeNormal( inout float3 N, in uint Face )
{
	uint Axis = Face >> 1;
	
	float MaxAbs = 1.0 / sqrt(2.0);

	N.xy = N.xy * (2 * MaxAbs) - (1 * MaxAbs);
	N.z = sqrt( 1 - dot( N.xy, N.xy ) );

	N = Axis == 0 ? N.zxy : N;
	N = Axis == 1 ? N.xzy : N;
	N *= (Face & 1) ? -1 : 1;
}

float3 EncodeBaseColor(float3 BaseColor)
{
	// we use sRGB on the render target to give more precision to the darks
	return BaseColor;
}

float3 DecodeBaseColor(float3 BaseColor)
{
	// we use sRGB on the render target to give more precision to the darks
	return BaseColor;
}

float3 EncodeSubsurfaceColor(float3 SubsurfaceColor)
{
	return sqrt(saturate(SubsurfaceColor));
}

// @param SubsurfaceProfile 0..1, SubsurfaceProfileId = int(x * 255)
float3 EncodeSubsurfaceProfile(float SubsurfaceProfile)
{
	return float3(SubsurfaceProfile, 0, 0);
}

// Derive density from a heuristic using opacity, tweaked for useful falloff ranges and to give a linear depth falloff with opacity
float SubsurfaceDensityFromOpacity(float Opacity)
{
	return (-0.05f * log(1.0f - min(Opacity, 0.99f)));
}

float EncodeIndirectIrradiance(float IndirectIrradiance)
{
	float L = IndirectIrradiance;
	L *= View.PreExposure; // Apply pre-exposure as a mean to prevent compression overflow.
	const float LogBlackPoint = 0.00390625;	// exp2(-8);
	return log2( L + LogBlackPoint ) / 16 + 0.5;
}

float DecodeIndirectIrradiance(float IndirectIrradiance)
{
	// LogL -> L
	float LogL = IndirectIrradiance;
	const float LogBlackPoint = 0.00390625;	// exp2(-8);
	return View.OneOverPreExposure * (exp2( LogL * 16 - 8 ) - LogBlackPoint);	// 1 exp2, 1 smad, 1 ssub
}

float4 EncodeWorldTangentAndAnisotropy(float3 WorldTangent, float Anisotropy)
{
	return float4(
		EncodeNormal(WorldTangent),
		Anisotropy * 0.5f + 0.5f
		);
}

float ComputeAngleFromRoughness( float Roughness, const float Threshold = 0.04f )
{
#if 1
	float Angle = 3 * Square( Roughness );
#else
	const float LogThreshold = log2( Threshold );
	float Power = 0.5 / pow( Roughness, 4 ) - 0.5;
	float Angle = acos( exp2( LogThreshold / Power ) );
#endif
	return Angle;
}

float ComputeRoughnessFromAngle( float Angle, const float Threshold = 0.04f )
{
#if 1
	float Roughness = sqrt( 0.33333 * Angle );
#else
	const float LogThreshold = log2( Threshold );
	float Power = LogThreshold / log2( cos( Angle ) );
	float Roughness = sqrt( sqrt( 2 / (Power * 4 + 2) ) );
#endif
	return Roughness;
}

float AddAngleToRoughness( float Angle, float Roughness )
{
	return saturate( sqrt( Square( Roughness ) + 0.33333 * Angle ) );
}

// @param Scalar clamped in 0..1 range
// @param Mask 0..1
// @return 8bit in range 0..1
float Encode71(float Scalar, uint Mask)
{
	return
		127.0f / 255.0f * saturate(Scalar) +
		128.0f / 255.0f * Mask;
}

// 8bit reinterpretation as 7bit,1bit
// @param Scalar 0..1
// @param Mask 0..1
// @return 7bit in 0.1
float Decode71(float Scalar, out uint Mask)
{
	Mask = (uint)(Scalar > 0.5f);

	return (Scalar - 0.5f * Mask) * 2.0f;
}

float EncodeShadingModelIdAndSelectiveOutputMask(uint ShadingModelId, uint SelectiveOutputMask)
{
	uint Value = (ShadingModelId & SHADINGMODELID_MASK) | SelectiveOutputMask;
	return (float)Value / (float)0xFF;
}

uint DecodeShadingModelId(float InPackedChannel)
{
	return ((uint)round(InPackedChannel * (float)0xFF)) & SHADINGMODELID_MASK;
}

uint DecodeSelectiveOutputMask(float InPackedChannel)
{
	return ((uint)round(InPackedChannel * (float)0xFF)) & ~SHADINGMODELID_MASK;
}

bool IsEyeModel(int ShadingModel)
{
#if FORCE_SIMPLE_SHADING || SUBSTRATE_FASTPATH
	return false;
#else
	return ShadingModel == SHADINGMODELID_EYE;
#endif
}

bool IsSubsurfaceModel(int ShadingModel)
{
#if FORCE_SIMPLE_SHADING || SUBSTRATE_FASTPATH
	return false;
#else
	return ShadingModel == SHADINGMODELID_SUBSURFACE 
		|| ShadingModel == SHADINGMODELID_PREINTEGRATED_SKIN 
		|| ShadingModel == SHADINGMODELID_SUBSURFACE_PROFILE
		|| ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE
		|| ShadingModel == SHADINGMODELID_HAIR
		|| ShadingModel == SHADINGMODELID_EYE
		|| ShadingModel == SHADINGMODELID_SUBSTRATE_TOON; // For shadow transmittance similar to Subsurface.
#endif
}

bool UseSubsurfaceProfile(int ShadingModel)
{
#if FORCE_SIMPLE_SHADING || SUBSTRATE_FASTPATH
	return false;
#else
	return ShadingModel == SHADINGMODELID_SUBSURFACE_PROFILE 
		|| ShadingModel == SHADINGMODELID_EYE;
#endif
}

bool HasCustomGBufferData(int ShadingModelID)
{
#if FORCE_SIMPLE_SHADING || SUBSTRATE_FASTPATH
	return false;
#else
	return ShadingModelID == SHADINGMODELID_SUBSURFACE
		|| ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN
		|| ShadingModelID == SHADINGMODELID_CLEAR_COAT
		|| ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE
		|| ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE
		|| ShadingModelID == SHADINGMODELID_HAIR
		|| ShadingModelID == SHADINGMODELID_CLOTH
		|| ShadingModelID == SHADINGMODELID_EYE
		|| ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON;
#endif
}

bool HasAnisotropy(int SelectiveOutputMask)
{
	return (SelectiveOutputMask & HAS_ANISOTROPY_MASK) != 0;
}

// When using substrate and a material that is legacy prlugged on the root node (not using front input), we need to skip writting anisotropy data.
// That is because we use substrate unified lighting that supported anisotropy everywhere. So we have to disable anisotropy explicitely for some legacy shading model that did not support.
// This in turn will result in correct classification when reading the gbuffer and skip some tiles that would have been complex with anisotropy to only be single/simple now.
bool LegacyShadingModelSupportsAnisotropy(uint ShadingModelID)
{
	// See ShadingModels.ush for where this comes from.
	return ShadingModelID == SHADINGMODELID_DEFAULT_LIT
		|| ShadingModelID == SHADINGMODELID_CLEAR_COAT
		|| ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON;
}

// all values that are output by the forward rendering pass
struct FGBufferData
{
	// normalized
	half3 WorldNormal;
	// normalized, only valid if HAS_ANISOTROPY_MASK in SelectiveOutputMask
	half3 WorldTangent;
	// 0..1 (derived from BaseColor, Metalness, Specular)
	half3 DiffuseColor;
	// 0..1 (derived from BaseColor, Metalness, Specular)
	half3 SpecularColor;
	// 0..1, white for SHADINGMODELID_SUBSURFACE_PROFILE and SHADINGMODELID_EYE (apply BaseColor after scattering is more correct and less blurry)
	half3 BaseColor;
	// 0..1
	half Metallic;
	// 0..1
	half Specular;
	// 0..1
	float4 CustomData;
	// AO utility value
	half GenericAO;
	// Indirect irradiance luma
	half IndirectIrradiance;
	// Static shadow factors for channels assigned by Lightmass
	// Lights using static shadowing will pick up the appropriate channel in their deferred pass
	half4 PrecomputedShadowFactors;
	// 0..1
	half Roughness;
	// -1..1, only valid if only valid if HAS_ANISOTROPY_MASK in SelectiveOutputMask
	half Anisotropy;
	// 0..1 ambient occlusion  e.g.SSAO, wet surface mask, skylight mask, ...
	half GBufferAO;
	// Bit mask for occlusion of the diffuse indirect samples
	uint DiffuseIndirectSampleOcclusion;
	// 0..255 
	uint ShadingModelID;
	// 0..255 
	uint SelectiveOutputMask;
	// 0..1, 2 bits, use CastContactShadow(GBuffer) or HasDynamicIndirectShadowCasterRepresentation(GBuffer) to extract
	half PerObjectGBufferData;
	// in world units
	float CustomDepth;
	// Custom depth stencil value
	uint CustomStencil;
	// in unreal units (linear), can be used to reconstruct world position,
	// only valid when decoding the GBuffer as the value gets reconstructed from the Z buffer
	float Depth;
	// Velocity for motion blur (only used when WRITES_VELOCITY_TO_GBUFFER is enabled)
	half4 Velocity;

	// 0..1, only needed by SHADINGMODELID_SUBSURFACE_PROFILE and SHADINGMODELID_EYE which apply BaseColor later
	half3 StoredBaseColor;
	// 0..1, only needed by SHADINGMODELID_SUBSURFACE_PROFILE and SHADINGMODELID_EYE which apply Specular later
	half StoredSpecular;
	// 0..1, only needed by SHADINGMODELID_EYE which encodes Iris Distance inside Metallic
	half StoredMetallic;
	// Only needed for SHADINGMODELID_TWOSIDED_FOLIAGE
	half3 NormalDistribution;

	// Curvature for mobile subsurface profile
	half Curvature;
};

bool CastContactShadow(FGBufferData GBufferData)
{
	uint PackedAlpha = (uint)(GBufferData.PerObjectGBufferData * 3.999f);
	bool bCastContactShadowBit = PackedAlpha & 1;
	// Exclude eye materials from ever casting contact shadows
	bool bShadingModelCastContactShadows = (GBufferData.ShadingModelID != SHADINGMODELID_EYE);
	return bCastContactShadowBit && bShadingModelCastContactShadows;
}

bool HasDynamicIndirectShadowCasterRepresentation(FGBufferData GBufferData)
{
	uint PackedAlpha = (uint)(GBufferData.PerObjectGBufferData * 3.999f);
	return (PackedAlpha & 2) != 0;
}

bool IsFirstPerson(FGBufferData GBufferData)
{
#if !ALLOW_STATIC_LIGHTING
	return (GBufferData.SelectiveOutputMask & IS_FIRST_PERSON_MASK) != 0;
#else
	return false;
#endif
}

bool HasPrecShadowMask(FGBufferData GBufferData)
{
#if ALLOW_STATIC_LIGHTING
	return !(GBufferData.SelectiveOutputMask & SKIP_PRECSHADOW_MASK);
#else
	return false;
#endif
}

bool HasZeroPrecShadowMask(FGBufferData GBufferData)
{
#if ALLOW_STATIC_LIGHTING
	return GBufferData.SelectiveOutputMask & ZERO_PRECSHADOW_MASK;
#else
	return false;
#endif
}

// High frequency Checkerboard pattern
// @param PixelPos relative to left top of the rendertarget (not viewport)
// @return true/false, todo: profile if float 0/1 would be better (need to make sure it's 100% the same)
bool CheckerFromPixelPos(uint2 PixelPos)
{
	// todo: Index is float and by staying float we can optimize this 
	// We alternate the pattern to get 2x supersampling on the lower res data to get more near to full res
	uint TemporalAASampleIndex = uint(View.TemporalAAParams.x);

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	return (PixelPos.x + PixelPos.y + TemporalAASampleIndex) % 2;
#else
	return (uint)(fmod(PixelPos.x + PixelPos.y + TemporalAASampleIndex, 2)) != 0;
#endif
}

// High frequency Checkerboard pattern
// @param UVSceneColor at pixel center
// @return true/false, todo: profile if float 0/1 would be better (need to make sure it's 100% the same)
bool CheckerFromSceneColorUV(float2 UVSceneColor)
{
	// relative to left top of the rendertarget (not viewport)
	uint2 PixelPos = uint2(UVSceneColor * View.BufferSizeAndInvSize.xy);

	return CheckerFromPixelPos(PixelPos);
}

#if GBUFFER_REFACTOR
#include "GBufferHelpers.ush"
	#if FEATURE_LEVEL > FEATURE_LEVEL_ES3_1
	#include "/Engine/Generated/ShaderAutogen/AutogenShaderHeaders.ush"
	#endif
#endif

struct FScreenSpaceData
{
	// GBuffer (material attributes from forward rendering pass)
	FGBufferData GBuffer;
	// 0..1, only valid in some passes, 1 if off
	float AmbientOcclusion;
};

/** Sets up the Gbuffer for an unlit material. */
void SetGBufferForUnlit(out float4 OutGBufferB)
{
	OutGBufferB = 0;
	OutGBufferB.a = EncodeShadingModelIdAndSelectiveOutputMask(SHADINGMODELID_UNLIT, 0);
}

#define INDIRECT_SAMPLE_COUNT 8

float4 ComputeIndirectLightingSampleE(uint2 TracingPixelCoord, uint TracingRayIndex, uint TracingRayCount)
{
	//  Shader compiler code is pretty good at moving this out of any for loops.
	uint2 Seed0 = Rand3DPCG16(int3(TracingPixelCoord, View.StateFrameIndexMod8)).xy;
	uint2 Seed1 = Rand3DPCG16(int3(TracingPixelCoord + 17, View.StateFrameIndexMod8)).xy;

	return float4(
		Hammersley16(TracingRayIndex, TracingRayCount, Seed0),
		Hammersley16(TracingRayIndex, TracingRayCount, Seed1));
}

#if SHADING_PATH_MOBILE

#ifndef ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED
#define ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED 1
#endif

#define MOBILE_SHADINGMODEL_SUPPORT (!MOBILE_DEFERRED_SHADING || ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED)

float MobileEncodeIdAndColorChannel(uint Id, float Color, bool b10Bits)
{
	if (b10Bits)
	{
		uint ColorValue = (uint)round(clamp(Color, 0.0, 1.0) * 63.0) << 4; // 6bit color
		return (float)(ColorValue | (Id & 0x0F)) / 1023.0;
	}
	else // 8 bits
	{
		uint ColorValue = (uint)round(clamp(Color, 0.0, 1.0) * 31.0) << 3; // 5bit color
		return (float)(ColorValue | (Id & 0x07)) / 255.0;
	}
}

uint MobileDecodeId(float InPackedChannel, bool b10Bits)
{
	if (b10Bits)
	{
		return ((uint)round(InPackedChannel * 1023.0)) & 0x0F;
	}
	else // 8 bits
	{
		return ((uint)round(InPackedChannel * 255.0)) & 0x07;
	}
}

float MobileDecodeColorChannel(float InPackedChannel, bool b10Bits)
{
	if (b10Bits)
	{
		return ((uint)(InPackedChannel * 1023.0) >> 4) / 63.0;
	}
	else // 8 bits
	{
		return ((uint)(InPackedChannel * 255.0) >> 3) / 31.0;
	}
}

/** Mobile specific encoding of GBuffer data */
void MobileEncodeGBuffer(
	FGBufferData GBuffer,
	out half4 OutGBufferA,
	out half4 OutGBufferB,
	out half4 OutGBufferC,
	out half4 OutGBufferD
)
{
	if (GBuffer.ShadingModelID == SHADINGMODELID_UNLIT)
	{
		OutGBufferA = 0;
		OutGBufferB = 0;
		OutGBufferC = 0;
		OutGBufferD = 0;
	}
	else
	{
#if	MOBILE_EXTENDED_GBUFFER && !ALLOW_STATIC_LIGHTING
		OutGBufferA.rgb = EncodeNormal(GBuffer.WorldNormal);
#else
		OutGBufferA.rg = UnitVectorToOctahedron(normalize(GBuffer.WorldNormal)) * 0.5 + 0.5;
		#if ALLOW_STATIC_LIGHTING
			OutGBufferA.b = EncodeIndirectIrradiance(GBuffer.IndirectIrradiance);
		#else
			OutGBufferA.b = 1;
		#endif
#endif
		OutGBufferA.a = GBuffer.PerObjectGBufferData;		

		OutGBufferB.r = GBuffer.Metallic;
		OutGBufferB.g = GBuffer.Specular;
		// GBuffer can be overwritten by decals and others. See MobileBasePassPixelShader.usf for this clamp
		OutGBufferB.b = max(0.015625, GBuffer.Roughness);
		OutGBufferB.a = GBuffer.ShadingModelID / 255.0;

		OutGBufferC.rgb = EncodeBaseColor( GBuffer.BaseColor );
#if ALLOW_STATIC_LIGHTING
		OutGBufferC.a = GBuffer.PrecomputedShadowFactors.x;
#else
		OutGBufferC.a = GBuffer.GBufferAO;
#endif
		OutGBufferD = GBuffer.CustomData;

#if MOBILE_SHADINGMODEL_SUPPORT
	#if	MOBILE_EXTENDED_GBUFFER
		if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
		{
			// encode Curvature into 5 bits and 3 bits for SubsurfaceProfile 
			OutGBufferD.x = MobileEncodeIdAndColorChannel(GBuffer.CustomData.x, GBuffer.Curvature, false);
		}
	#else
		if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || 
			GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
		{
			// encode Opacity into 6 bits and 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, GBuffer.CustomData.a, true);
			OutGBufferB.a = GBuffer.CustomData.r; // SubsurfaceColor.R
			OutGBufferC.a = GBuffer.CustomData.g; // SubsurfaceColor.G
			OutGBufferB.r = GBuffer.CustomData.b; // SubsurfaceColor.B
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
		{
			// encode 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, 0, true);
			OutGBufferB.a = GBuffer.CustomData.x; // ClearCoat
			OutGBufferC.a = GBuffer.CustomData.y; // ClearCoatRoughness
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE)
		{
			// encode Opacity into 6 bits and 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, GBuffer.CustomData.a, true);
			OutGBufferB.a = GBuffer.CustomData.r; // SubsurfaceProfile
			OutGBufferC.a = GBuffer.Curvature; // Curvature
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			// encode Backlit into 6 bits and 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, GBuffer.CustomData.z, true);
			OutGBufferB.a = GBuffer.CustomData.x; // 
			OutGBufferC.a = GBuffer.CustomData.y; // 
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
		{
			// encode 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, 0, true);
			OutGBufferB.a = GBuffer.CustomData.r; // SubsurfaceColor.R
			OutGBufferC.a = GBuffer.CustomData.g; // SubsurfaceColor.G
			OutGBufferB.r = GBuffer.CustomData.b; // SubsurfaceColor.B
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
		{
			// encode Cloth into 6 bits and 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, GBuffer.CustomData.a, true);
			OutGBufferB.a = GBuffer.CustomData.r; // SubsurfaceColor.R
			OutGBufferC.a = GBuffer.CustomData.g; // SubsurfaceColor.G
			OutGBufferB.r = GBuffer.CustomData.b; // SubsurfaceColor.B
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
		{
			// encode IrisMask into 6 bits and 4 bits for shading model 
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, GBuffer.CustomData.w, true);
			// encode Curvature into 5 bits and 3 bits for SubsurfaceProfile 
			OutGBufferB.a = MobileEncodeIdAndColorChannel(GBuffer.CustomData.x, GBuffer.Curvature, false);
			OutGBufferC.a = GBuffer.CustomData.y; // IrisNormal_Oct.x
			OutGBufferB.g = GBuffer.CustomData.z; // IrisNormal_Oct.y
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
		{
			// This is the toon profile index.
			OutGBufferC.a = GBuffer.CustomData.a;
			// UVs are ignored and ramp offset or shadow hatching won't be used on mobile.

			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, 0, true);
			// OutGBufferA.b and OutGBufferB.a could be used to store more data.
		}
		else
		{
			// encode 4 bits for shading model
			OutGBufferA.b = MobileEncodeIdAndColorChannel(GBuffer.ShadingModelID, 0, true);
		}
	#endif
#endif
	}
}

FGBufferData MobileDecodeGBuffer(half4 InGBufferA, half4 InGBufferB, half4 InGBufferC, half4 InGBufferD)
{
	FGBufferData GBuffer = (FGBufferData)0;

#if	MOBILE_EXTENDED_GBUFFER && !ALLOW_STATIC_LIGHTING		
	GBuffer.WorldNormal = normalize(DecodeNormal(InGBufferA.rgb));
#else
	GBuffer.WorldNormal = OctahedronToUnitVector(InGBufferA.rg * 2.0 - 1.0);
	#if ALLOW_STATIC_LIGHTING
		GBuffer.IndirectIrradiance = DecodeIndirectIrradiance(InGBufferA.b);
	#else
		GBuffer.IndirectIrradiance = 1;
	#endif
#endif
	GBuffer.PerObjectGBufferData = InGBufferA.a;

	GBuffer.Metallic	= InGBufferB.r;
	GBuffer.Specular	= InGBufferB.g;
	GBuffer.Roughness	= InGBufferB.b;
	// Note: must match GetShadingModelId standalone function logic
	// Also Note: SimpleElementPixelShader directly sets SV_Target2 ( GBufferB ) to indicate unlit.
	// An update there will be required if this layout changes.
	GBuffer.ShadingModelID = MOBILE_SHADINGMODEL_SUPPORT ? (uint)round(InGBufferB.a * 255.0f) : SHADINGMODELID_DEFAULT_LIT;
	GBuffer.SelectiveOutputMask = 0;
	GBuffer.BaseColor = DecodeBaseColor(InGBufferC.rgb);
#if ALLOW_STATIC_LIGHTING
	GBuffer.GBufferAO = 1;
	GBuffer.PrecomputedShadowFactors = half4(InGBufferC.a, 1, 1, 1);
#else
	GBuffer.GBufferAO = InGBufferC.a;
	GBuffer.PrecomputedShadowFactors = 1.0;
#endif

	GBuffer.StoredBaseColor = GBuffer.BaseColor;
	GBuffer.StoredMetallic = GBuffer.Metallic;
	GBuffer.StoredSpecular = GBuffer.Specular;

#if MOBILE_SHADINGMODEL_SUPPORT
	#if	MOBILE_EXTENDED_GBUFFER
		GBuffer.CustomData = HasCustomGBufferData(GBuffer.ShadingModelID) ? InGBufferD : 0;
		if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
		{
			GBuffer.Curvature = MobileDecodeColorChannel(InGBufferD.x, false); // Curvature
		}
	#else
		GBuffer.ShadingModelID = MobileDecodeId(InGBufferA.b, true);

		if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE ||
			GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
		{
			GBuffer.CustomData.a = MobileDecodeColorChannel(InGBufferA.b, true); // Opacity
			GBuffer.CustomData.r = InGBufferB.a; // SubsurfaceColor.R
			GBuffer.CustomData.g = InGBufferC.a; // SubsurfaceColor.G
			GBuffer.CustomData.b = InGBufferB.r; // SubsurfaceColor.B
			GBuffer.GBufferAO = 1;	// no space for AO
			GBuffer.Metallic = 0;	// no space for Metallic
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
		{
			GBuffer.CustomData.x = InGBufferB.a; // ClearCoat
			GBuffer.CustomData.y = InGBufferC.a; // ClearCoatRoughness
			GBuffer.GBufferAO = 1;	// no space for AO
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE)
		{
			GBuffer.CustomData.a = MobileDecodeColorChannel(InGBufferA.b, true); // Opacity
			GBuffer.CustomData.r = InGBufferB.a; // SubsurfaceProfile
			GBuffer.Curvature = InGBufferC.a; // Curvature
			GBuffer.GBufferAO = 1;	// no space for AO
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			GBuffer.CustomData.z = MobileDecodeColorChannel(InGBufferA.b, true); // Backlit
			GBuffer.CustomData.x = InGBufferB.a; // 
			GBuffer.CustomData.y = InGBufferC.a; //
			GBuffer.GBufferAO = 1;	// no space for AO
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
		{
			GBuffer.CustomData.r = InGBufferB.a; // SubsurfaceColor.R
			GBuffer.CustomData.g = InGBufferC.a; // SubsurfaceColor.G
			GBuffer.CustomData.b = InGBufferB.r; // SubsurfaceColor.B
			GBuffer.GBufferAO = 1;	// no space for AO
			GBuffer.Metallic = 0;	// no space for Metallic
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
		{
			GBuffer.CustomData.a = MobileDecodeColorChannel(InGBufferA.b, true); // Cloth
			GBuffer.CustomData.r = InGBufferB.a; // SubsurfaceColor.R
			GBuffer.CustomData.g = InGBufferC.a; // SubsurfaceColor.G
			GBuffer.CustomData.b = InGBufferB.r; // SubsurfaceColor.B
			GBuffer.GBufferAO = 1;	// no space for AO
			GBuffer.Metallic = 0;	// no space for Metallic
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
		{
			GBuffer.CustomData.w = MobileDecodeColorChannel(InGBufferA.b, true); // IrisMask
			GBuffer.CustomData.x = MobileDecodeId(InGBufferB.a, false); // SubsurfaceProfile
			GBuffer.CustomData.y = InGBufferC.a; // IrisNormal_Oct.x
			GBuffer.CustomData.z = InGBufferB.g; // IrisNormal_Oct.y
			GBuffer.Curvature = MobileDecodeColorChannel(InGBufferB.a, false); // Curvature
			
			GBuffer.GBufferAO = 1;	// no space for AO
			GBuffer.Specular = 0.5f;	// no space for Specular
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
		{
			// This is the toon profile index.
			GBuffer.CustomData.a = InGBufferC.a;
			// UVs are ignored and ramp offset or shadow hatching won't be used on mobile.
			GBuffer.CustomData.rgb = 0.0f.xxx;

			// OutGBufferA.b and OutGBufferB.a could be used to store more data.

			GBuffer.GBufferAO = 1;	// no space for AO
		}
	#endif

	if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
	{
		GBuffer.Metallic = 0.0;
	#if IRIS_NORMAL
		GBuffer.Specular = 0.25;
	#endif
	}
#endif

	// derived from BaseColor, Metalness, Specular
	{
		GBuffer.SpecularColor = ComputeF0(GBuffer.Specular, GBuffer.BaseColor, GBuffer.Metallic);
		GBuffer.DiffuseColor = GBuffer.BaseColor - GBuffer.BaseColor * GBuffer.Metallic;
	}

#if USE_DEVELOPMENT_SHADERS
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		// These values are set the by the FSetupViewForRendering of the ViewportClient
		GBuffer.DiffuseColor = GBuffer.DiffuseColor * View.DiffuseOverrideParameter.www + View.DiffuseOverrideParameter.xyz;
		GBuffer.SpecularColor = GBuffer.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
	}
#endif

	return GBuffer;
}

#if MOBILE_DEFERRED_SHADING

#define	FRAMEBUFFER_FETCH_ACCESS (IS_MOBILE_DEFERREDSHADING_SUBPASS && PIXELSHADER && ALLOW_FRAMEBUFFER_FETCH)

#if (METAL_ES3_1_PROFILE && FRAMEBUFFER_FETCH_ACCESS)
#include "/Engine/Public/Platform/Metal/MetalSubpassSupport.ush"
#elif (VULKAN_PROFILE && FRAMEBUFFER_FETCH_ACCESS)
#include "/Engine/Public/Platform/Vulkan/VulkanSubpassSupport.ush"
#elif (USE_GLES_FBF_DEFERRED && FRAMEBUFFER_FETCH_ACCESS)
#include "/Engine/Public/Platform/GL/GLSubpassSupport.ush"
#endif

void MobileFetchGBuffer(in float2 UV, out half4 GBufferA, out half4 GBufferB, out half4 GBufferC, out half4 GBufferD, out float SceneDepth)
{
#if (VULKAN_PROFILE && FRAMEBUFFER_FETCH_ACCESS)
	GBufferA = VulkanSubpassFetch1(); 
	GBufferB = VulkanSubpassFetch2(); 
	GBufferC = VulkanSubpassFetch3();
	#if MOBILE_EXTENDED_GBUFFER
		GBufferD = VulkanSubpassFetch4();
	#else
		GBufferD = 0;
	#endif
#elif (METAL_ES3_1_PROFILE && !MAC && FRAMEBUFFER_FETCH_ACCESS)
	GBufferA = SubpassFetchRGBA_1(); 
	GBufferB = SubpassFetchRGBA_2(); 
	GBufferC = SubpassFetchRGBA_3(); 
	#if MOBILE_EXTENDED_GBUFFER
		GBufferD = SubpassFetchRGBA_4();
	#else
		GBufferD = 0;
	#endif
#elif (USE_GLES_FBF_DEFERRED && FRAMEBUFFER_FETCH_ACCESS)
	GBufferA = GLSubpassFetch1(); 
	GBufferB = GLSubpassFetch2(); 
	GBufferC = GLSubpassFetch3();  
	GBufferD = 0; // PLS is limited to 128bits
#else
	GBufferA = Texture2DSampleLevel(MobileSceneTextures.GBufferATexture, MobileSceneTextures.GBufferATextureSampler, UV, 0); 
	GBufferB = Texture2DSampleLevel(MobileSceneTextures.GBufferBTexture, MobileSceneTextures.GBufferBTextureSampler, UV, 0);
	GBufferC = Texture2DSampleLevel(MobileSceneTextures.GBufferCTexture, MobileSceneTextures.GBufferCTextureSampler, UV, 0);
	#if MOBILE_EXTENDED_GBUFFER
		GBufferD = Texture2DSampleLevel(MobileSceneTextures.GBufferDTexture, MobileSceneTextures.GBufferDTextureSampler, UV, 0);
	#else
		GBufferD = 0;
	#endif
#endif

#if	(FRAMEBUFFER_FETCH_ACCESS && VULKAN_PROFILE)
	SceneDepth = ConvertFromDeviceZ(VulkanSubpassDepthFetch());
#elif (FRAMEBUFFER_FETCH_ACCESS && !MAC && (USE_SCENE_DEPTH_AUX || USE_GLES_FBF_DEFERRED))
	SceneDepth = ConvertFromDeviceZ(DepthbufferFetchES2());
#else
	SceneDepth = ConvertFromDeviceZ(MobileSceneTextures.SceneDepthTexture.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, UV, 0).r);
#endif
}
#endif //MOBILE_DEFERRED_SHADING

FGBufferData MobileFetchAndDecodeGBuffer(in float2 UV, in float2 PixelPos, uint ViewId)
{
	FGBufferData GBuffer = (FGBufferData)0;
#if MOBILE_DEFERRED_SHADING
	float SceneDepth = 0; 
	half4 GBufferA = 0;
	half4 GBufferB = 0;
	half4 GBufferC = 0;
	half4 GBufferD = 0;
	MobileFetchGBuffer(UV, GBufferA, GBufferB, GBufferC, GBufferD, SceneDepth);
	GBuffer = MobileDecodeGBuffer(GBufferA, GBufferB, GBufferC, GBufferD);
	GBuffer.Depth = SceneDepth;
#else
	GBuffer.Depth = CalcSceneDepth(UV);
#endif
#if	MOBILE_MULTI_VIEW
	GBuffer.CustomDepth = ConvertFromDeviceZ(MobileSceneTextures.CustomDepthTextureArray.Load(int4(PixelPos.xy, ViewId, 0)).r);
	GBuffer.CustomStencil = MobileSceneTextures.CustomStencilTextureArray.Load(int4(PixelPos.xy, ViewId, 0)) STENCIL_COMPONENT_SWIZZLE;
#else
	GBuffer.CustomDepth = ConvertFromDeviceZ(MobileSceneTextures.CustomDepthTexture.Load(int3(PixelPos.xy, 0)).r);
	GBuffer.CustomStencil = MobileSceneTextures.CustomStencilTexture.Load(int3(PixelPos.xy, 0)) STENCIL_COMPONENT_SWIZZLE;
#endif


	return GBuffer;
}

FGBufferData MobileFetchAndDecodeGBuffer(in float2 UV, in float2 PixelPos)
{
	return MobileFetchAndDecodeGBuffer(UV, PixelPos, 0);
}

// Mirror desktop GetGBufferData() 
FGBufferData GetGBufferData(float2 UV, bool bGetNormalizedNormal = true, bool bForceUnlitOrDefaultLit = false)
{
	float2 PixelPos = trunc(UV * View.BufferSizeAndInvSize.xy * View.BufferToSceneTextureScale.xy);
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UV, PixelPos);
	if (bGetNormalizedNormal)
	{
		GBuffer.WorldNormal = normalize(GBuffer.WorldNormal);
	}
	return GBuffer;
}

#endif //SHADING_PATH_MOBILE

// UE_DEPRECATED 5.7 - Deprecated by Substrate
/** Populates OutGBufferA, B and C */
void EncodeGBuffer(
	FGBufferData GBuffer,
	out float4 OutGBufferA,
	out float4 OutGBufferB,
	out float4 OutGBufferC,
	out float4 OutGBufferD,
	out float4 OutGBufferE,
	out float4 OutGBufferVelocity,
	float QuantizationBias = 0		// -0.5 to 0.5 random float. Used to bias quantization.
	)
{
#if SHADING_PATH_MOBILE
	MobileEncodeGBuffer(GBuffer, OutGBufferA, OutGBufferB, OutGBufferC, OutGBufferD);
	OutGBufferE = 0;
	OutGBufferVelocity = 0;
#else
	if (GBuffer.ShadingModelID == SHADINGMODELID_UNLIT)
	{
		OutGBufferA = 0;
		SetGBufferForUnlit(OutGBufferB);
		OutGBufferC = 0;
		OutGBufferD = 0;
		OutGBufferE = 0;
	}
	else
	{
#if 1
		OutGBufferA.rgb = EncodeNormal( GBuffer.WorldNormal );
		OutGBufferA.a = GBuffer.PerObjectGBufferData;
#else
		float3 Normal = GBuffer.WorldNormal;
		uint   NormalFace = 0;
		EncodeNormal( Normal, NormalFace );

		OutGBufferA.rg = Normal.xy;
		OutGBufferA.b = 0;
		OutGBufferA.a = GBuffer.PerObjectGBufferData;
#endif

		OutGBufferB.r = GBuffer.Metallic;
		OutGBufferB.g = GBuffer.Specular;
		OutGBufferB.b = GBuffer.Roughness;
		OutGBufferB.a = EncodeShadingModelIdAndSelectiveOutputMask(GBuffer.ShadingModelID, GBuffer.SelectiveOutputMask);

		OutGBufferC.rgb = EncodeBaseColor( GBuffer.BaseColor );

#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
		OutGBufferC.a = float(GBuffer.DiffuseIndirectSampleOcclusion) * (1.0f / 255.0f);
#elif ALLOW_STATIC_LIGHTING
		// No space for AO. Multiply IndirectIrradiance by AO instead of storing.
		OutGBufferC.a = EncodeIndirectIrradiance(GBuffer.IndirectIrradiance * GBuffer.GBufferAO) + QuantizationBias * (1.0 / 255.0);
#else
		OutGBufferC.a = GBuffer.GBufferAO;
#endif

		OutGBufferD = GBuffer.CustomData;
		OutGBufferE = GBuffer.PrecomputedShadowFactors;
	}

#if WRITES_VELOCITY_TO_GBUFFER
	OutGBufferVelocity = GBuffer.Velocity;
#else
	OutGBufferVelocity = 0;
#endif
#endif// SHADING_PATH_MOBILE
}

// SubsurfaceProfile does deferred lighting with a checker board pixel pattern
// we separate the view from the non view dependent lighting and later recombine the two color constributions in a postprocess
// We have the option to apply the BaseColor/Specular in the base pass or do it later in the postprocess (has implications to texture detail, fresnel and performance)
bool AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(inout float3 BaseColor, inout float Specular, bool bChecker)
{
#if SUBSURFACE_CHANNEL_MODE == 0
	// If SUBSURFACE_CHANNEL_MODE is 0, we can't support full-resolution lighting, so we 
	// ignore View.bCheckerboardSubsurfaceProfileRendering
	const bool bCheckerboardRequired = View.bSubsurfacePostprocessEnabled > 0;
#else
	const bool bCheckerboardRequired = View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0 && ALLOW_SSS_MATERIAL_OVERRIDE && ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE;
	BaseColor = (ALLOW_SSS_MATERIAL_OVERRIDE && View.bSubsurfacePostprocessEnabled) ? float3(1, 1, 1) : BaseColor;
#endif
	if (bCheckerboardRequired)
	{
		// because we adjust the BaseColor here, we need StoredBaseColor
		// we apply the base color later in SubsurfaceRecombinePS()
		BaseColor = bChecker;
		Specular *= !bChecker;
	}
	return bCheckerboardRequired;
}
void AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(inout float3 BaseColor, inout float3 SpecularColor, inout float Specular, bool bChecker)
{
// If this function is called in translucent pass, we should not modify the BaseColor, instead directly read the base color as it is. 
// The translucent pass is called after the main post process subsurface scattering pass, it will not affect previous logic.
#if MATERIALBLENDING_TRANSLUCENT || !ALLOW_SSS_MATERIAL_OVERRIDE
	return;
#else
	const bool bCheckerboardRequired = AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(BaseColor, Specular, bChecker);
	if (bCheckerboardRequired)
	{
		// in SubsurfaceRecombinePS() does not multiply with Specular so we do it here
		SpecularColor *= !bChecker;
	}
#endif
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
/** Populates FGBufferData */
// @param bChecker High frequency Checkerboard pattern computed with one of the CheckerFrom.. functions, todo: profile if float 0/1 would be better (need to make sure it's 100% the same)
FGBufferData DecodeGBufferData(
	float4 InGBufferA,
	float4 InGBufferB,
	float4 InGBufferC,
	float4 InGBufferD,
	float4 InGBufferE,
	float4 InGBufferF,
	float4 InGBufferVelocity,
	float CustomNativeDepth,
	uint CustomStencil,
	float SceneDepth,
	bool bGetNormalizedNormal,
	bool bChecker)
{
	FGBufferData GBuffer;

	GBuffer.WorldNormal = DecodeNormal( InGBufferA.xyz );
	if(bGetNormalizedNormal)
	{
		GBuffer.WorldNormal = normalize(GBuffer.WorldNormal);
	}

	GBuffer.PerObjectGBufferData = InGBufferA.a;  
	GBuffer.Metallic	= InGBufferB.r;
	GBuffer.Specular	= InGBufferB.g;
	GBuffer.Roughness	= InGBufferB.b;
	// Note: must match GetShadingModelId standalone function logic
	// Also Note: SimpleElementPixelShader directly sets SV_Target2 ( GBufferB ) to indicate unlit.
	// An update there will be required if this layout changes.
	GBuffer.ShadingModelID = DecodeShadingModelId(InGBufferB.a);
	GBuffer.SelectiveOutputMask = DecodeSelectiveOutputMask(InGBufferB.a);

	GBuffer.BaseColor = DecodeBaseColor(InGBufferC.rgb);

#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
	GBuffer.DiffuseIndirectSampleOcclusion = 255 * InGBufferC.a;
	GBuffer.GBufferAO = saturate(1.0 - float(countbits(GBuffer.DiffuseIndirectSampleOcclusion)) * rcp(float(INDIRECT_SAMPLE_COUNT)));
	GBuffer.IndirectIrradiance = 1;
#elif ALLOW_STATIC_LIGHTING
	GBuffer.GBufferAO = 1;
	GBuffer.DiffuseIndirectSampleOcclusion = 0x0;
	GBuffer.IndirectIrradiance = DecodeIndirectIrradiance(InGBufferC.a);
#else
	GBuffer.GBufferAO = InGBufferC.a;
	GBuffer.DiffuseIndirectSampleOcclusion = 0x0;
	GBuffer.IndirectIrradiance = 1;
#endif

	GBuffer.CustomData = HasCustomGBufferData(GBuffer.ShadingModelID) ? InGBufferD : 0;

	// FirstPerson uses a bit in SelectiveOutputMask that is aliased with ZERO_PRECSHADOW_MASK when !ALLOW_STATIC_LIGHTING, so we explicitly skip this logic here.
#if ALLOW_STATIC_LIGHTING
	GBuffer.PrecomputedShadowFactors = HasPrecShadowMask(GBuffer) ? InGBufferE : (HasZeroPrecShadowMask(GBuffer) ? 0 : 1);
#else
	GBuffer.PrecomputedShadowFactors = half(1.0f);
#endif
	GBuffer.CustomDepth = ConvertFromDeviceZ(CustomNativeDepth);
	GBuffer.CustomStencil = CustomStencil;
	GBuffer.Depth = SceneDepth;

	GBuffer.StoredBaseColor = GBuffer.BaseColor;
	GBuffer.StoredMetallic = GBuffer.Metallic;
	GBuffer.StoredSpecular = GBuffer.Specular;

	FLATTEN
	if (IsEyeModel(GBuffer.ShadingModelID))
	{
		GBuffer.Metallic = 0.0;
#if IRIS_NORMAL
		GBuffer.Specular = 0.25;
#endif
	}

	// derived from BaseColor, Metalness, Specular
	{
		GBuffer.SpecularColor = ComputeF0(GBuffer.Specular, GBuffer.BaseColor, GBuffer.Metallic);

		if (UseSubsurfaceProfile(GBuffer.ShadingModelID))
		{
			AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(GBuffer.BaseColor, GBuffer.SpecularColor, GBuffer.Specular, bChecker);
		}

		GBuffer.DiffuseColor = GBuffer.BaseColor - GBuffer.BaseColor * GBuffer.Metallic;

		#if USE_DEVELOPMENT_SHADERS
		{
			// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
			GBuffer.DiffuseColor = GBuffer.DiffuseColor * View.DiffuseOverrideParameter.www + View.DiffuseOverrideParameter.xyz;
			GBuffer.SpecularColor = GBuffer.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
		}
		#endif //USE_DEVELOPMENT_SHADERS
	}

	{
		bool bHasAnisoProp = HasAnisotropy(GBuffer.SelectiveOutputMask);

		GBuffer.WorldTangent = bHasAnisoProp ? DecodeNormal(InGBufferF.rgb) : 0;
		GBuffer.Anisotropy   = bHasAnisoProp ? InGBufferF.a * 2.0f - 1.0f   : 0;

		if (bGetNormalizedNormal && bHasAnisoProp)
		{
			GBuffer.WorldTangent = normalize(GBuffer.WorldTangent);
		}
	}

	GBuffer.Velocity = !(GBuffer.SelectiveOutputMask & SKIP_VELOCITY_MASK) ? InGBufferVelocity : 0;

	GBuffer.NormalDistribution = 0.0;

	return GBuffer;
}

float3 ExtractSubsurfaceColor(FGBufferData BufferData)
{
	return Square(BufferData.CustomData.rgb);
}

uint ExtractSubsurfaceProfileInt(float ProfileNormFloat)
{
	return uint(ProfileNormFloat * 255.0f + 0.5f);
}

uint ExtractSubsurfaceProfileInt(FGBufferData BufferData)
{
	return ExtractSubsurfaceProfileInt(BufferData.CustomData.r);
}

float ConvertSubsurfaceProfileToFloat(uint InProfileIdUint)
{
	return (0xFF & InProfileIdUint) * (1.f / 255.f);
}

bool ShouldUseHairComplexTransmittance(FGBufferData GBuffer)
{
	return GBuffer.CustomData.a > 0;
}

#if SHADING_PATH_DEFERRED

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	// UE_DEPRECATED 5.7 - Deprecated by Substrate
	// @param PixelPos relative to left top of the rendertarget (not viewport)
	FGBufferData GetGBufferDataUint(uint2 PixelPos, bool bGetNormalizedNormal = true)
	{
	#if GBUFFER_REFACTOR
		return DecodeGBufferDataUint(PixelPos,bGetNormalizedNormal);
	#else 
		float4 GBufferA = SceneTexturesStruct.GBufferATexture.Load(int3(PixelPos, 0));
		float4 GBufferB = SceneTexturesStruct.GBufferBTexture.Load(int3(PixelPos, 0));
		float4 GBufferC = SceneTexturesStruct.GBufferCTexture.Load(int3(PixelPos, 0));
		float4 GBufferD = SceneTexturesStruct.GBufferDTexture.Load(int3(PixelPos, 0));
		float CustomNativeDepth = SceneTexturesStruct.CustomDepthTexture.Load(int3(PixelPos, 0)).r;
		uint CustomStencil = SceneTexturesStruct.CustomStencilTexture.Load(int3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;

		#if ALLOW_STATIC_LIGHTING
			float4 GBufferE = SceneTexturesStruct.GBufferETexture.Load(int3(PixelPos, 0));
		#else
			float4 GBufferE = 1;
		#endif

		float4 GBufferF = SceneTexturesStruct.GBufferFTexture.Load(int3(PixelPos, 0));

		#if WRITES_VELOCITY_TO_GBUFFER
			float4 GBufferVelocity = SceneTexturesStruct.GBufferVelocityTexture.Load(int3(PixelPos, 0));
		#else
			float4 GBufferVelocity = 0;
		#endif

		float SceneDepth = CalcSceneDepth(PixelPos);

		return DecodeGBufferData(GBufferA, GBufferB, GBufferC, GBufferD, GBufferE, GBufferF, GBufferVelocity, SGGX, CustomNativeDepth, CustomStencil, SceneDepth, bGetNormalizedNormal, CheckerFromPixelPos(PixelPos));
	#endif
	}

	// UE_DEPRECATED 5.7 - Deprecated by Substrate
	// @param PixelPos relative to left top of the rendertarget (not viewport)
	FScreenSpaceData GetScreenSpaceDataUint(uint2 PixelPos, bool bGetNormalizedNormal = true)
	{
		FScreenSpaceData Out;

		Out.GBuffer = GetGBufferDataUint(PixelPos, bGetNormalizedNormal);

		float4 ScreenSpaceAO = Texture2DSampleLevel(SceneTexturesStruct.ScreenSpaceAOTexture, SceneTexturesStruct_ScreenSpaceAOTextureSampler, (PixelPos + 0.5f) * View.BufferSizeAndInvSize.zw, 0);
		Out.AmbientOcclusion = ScreenSpaceAO.r;

		return Out;
	}
#endif

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// @param UV - UV space in the GBuffer textures (BufferSize resolution)
// TOOD: need to find a way to make this more convenient.
FGBufferData GetGBufferDataFromSceneTextures(float2 UV, bool bGetNormalizedNormal = true, bool bForceUnlitOrDefaultLit = false)
{
#if GBUFFER_REFACTOR
	return DecodeGBufferDataSceneTextures(UV,bGetNormalizedNormal,bForceUnlitOrDefaultLit);
#else
	float4 GBufferA = GBufferATexture.SampleLevel(GBufferATextureSampler, UV, 0);
	float4 GBufferB = GBufferBTexture.SampleLevel(GBufferBTextureSampler, UV, 0);
	float4 GBufferC = GBufferCTexture.SampleLevel(GBufferCTextureSampler, UV, 0);
	float4 GBufferD = GBufferDTexture.SampleLevel(GBufferDTextureSampler, UV, 0);
	float4 GBufferE = GBufferETexture.SampleLevel(GBufferETextureSampler, UV, 0);
	float4 GBufferF = GBufferFTexture.SampleLevel(GBufferFTextureSampler, UV, 0);
	float4 GBufferVelocity = GBufferVelocityTexture.SampleLevel(GBufferVelocityTextureSampler, UV, 0);

	uint CustomStencil = 0;
	float CustomNativeDepth = 0;

	float DeviceZ = SampleDeviceZFromSceneTextures(UV);

	float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	return DecodeGBufferData(GBufferA, GBufferB, GBufferC, GBufferD, GBufferE, GBufferF, GBufferVelocity, SGGX, CustomNativeDepth, CustomStencil, SceneDepth, bGetNormalizedNormal, CheckerFromSceneColorUV(UV));
#endif
}

/** Returns the light channel mask that should be executed for this pixel. */
uint GetSceneLightingChannel(uint2 PixelCoord)
{
	BRANCH
	if (bSceneLightingChannelsValid)
	{
		return SceneLightingChannels.Load(uint3(PixelCoord, 0)).x;
	}
	return ~0;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// @param UV - UV space in the GBuffer textures (BufferSize resolution)
FGBufferData GetGBufferData(float2 UV, bool bGetNormalizedNormal = true, bool bForceUnlitOrDefaultLit = false)
{
#if GBUFFER_REFACTOR
	return DecodeGBufferDataUV(UV,bGetNormalizedNormal,bForceUnlitOrDefaultLit);
#else
	float4 GBufferA = Texture2DSampleLevel(SceneTexturesStruct.GBufferATexture, SceneTexturesStruct_GBufferATextureSampler, UV, 0);
	float4 GBufferB = Texture2DSampleLevel(SceneTexturesStruct.GBufferBTexture, SceneTexturesStruct_GBufferBTextureSampler, UV, 0);
	float4 GBufferC = Texture2DSampleLevel(SceneTexturesStruct.GBufferCTexture, SceneTexturesStruct_GBufferCTextureSampler, UV, 0);
	float4 GBufferD = Texture2DSampleLevel(SceneTexturesStruct.GBufferDTexture, SceneTexturesStruct_GBufferDTextureSampler, UV, 0);
	float CustomNativeDepth = Texture2DSampleLevel(SceneTexturesStruct.CustomDepthTexture, SceneTexturesStruct_CustomDepthTextureSampler, UV, 0).r;

	// BufferToSceneTextureScale is necessary when translucent materials are rendered in a render target 
	// that has a different resolution than the scene color textures, e.g. r.SeparateTranslucencyScreenPercentage < 100.
	int2 IntUV = (int2)trunc(UV * View.BufferSizeAndInvSize.xy * View.BufferToSceneTextureScale.xy);
	uint CustomStencil = SceneTexturesStruct.CustomStencilTexture.Load(int3(IntUV, 0)) STENCIL_COMPONENT_SWIZZLE;

	#if ALLOW_STATIC_LIGHTING
		float4 GBufferE = Texture2DSampleLevel(SceneTexturesStruct.GBufferETexture, SceneTexturesStruct_GBufferETextureSampler, UV, 0);
	#else
		float4 GBufferE = 1;
	#endif

	float4 GBufferF = Texture2DSampleLevel(SceneTexturesStruct.GBufferFTexture, SceneTexturesStruct_GBufferFTextureSampler, UV, 0);

	#if WRITES_VELOCITY_TO_GBUFFER
		float4 GBufferVelocity = Texture2DSampleLevel(SceneTexturesStruct.GBufferVelocityTexture, SceneTexturesStruct_GBufferVelocityTextureSampler, UV, 0);
	#else
		float4 GBufferVelocity = 0;
	#endif

	float SceneDepth = CalcSceneDepth(UV);
	
	return DecodeGBufferData(GBufferA, GBufferB, GBufferC, GBufferD, GBufferE, GBufferF, GBufferVelocity, CustomNativeDepth, CustomStencil, SceneDepth, bGetNormalizedNormal, CheckerFromSceneColorUV(UV));
#endif
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// Minimal path for just the lighting model, used to branch around unlit pixels (skybox)
uint GetShadingModelId(float2 UV)
{
	return DecodeShadingModelId(Texture2DSampleLevel(SceneTexturesStruct.GBufferBTexture, SceneTexturesStruct_GBufferBTextureSampler, UV, 0).a);
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// @param UV - UV space in the GBuffer textures (BufferSize resolution)
FScreenSpaceData GetScreenSpaceData(float2 UV, bool bGetNormalizedNormal = true, bool bForceUnlitOrDefaultLit = false)
{
	FScreenSpaceData Out;

	Out.GBuffer = GetGBufferData(UV, bGetNormalizedNormal, bForceUnlitOrDefaultLit);
	float4 ScreenSpaceAO = Texture2DSampleLevel(SceneTexturesStruct.ScreenSpaceAOTexture, SceneTexturesStruct_ScreenSpaceAOTextureSampler, UV, 0);

	Out.AmbientOcclusion = ScreenSpaceAO.r;

	return Out;
}

#endif
// [ Jimenez et al. 2016, "Practical Realtime Strategies for Accurate Indirect Occlusion" ]
half3 AOMultiBounce(half3 BaseColor, half AO)
{
	half3 a = 2.0404 * BaseColor - 0.3324;
	half3 b = -4.7951 * BaseColor + 0.6417;
	half3 c = 2.7552 * BaseColor + 0.6903;
	return max(AO, ((AO * a + b) * AO + c) * AO);
}

#define LIGHTING_CHANNELS_TEXTURE_RAY_TRACING_REPRESENTATION_BIT	3
#define LIGHTING_CHANNELS_TEXTURE_CAST_SHADOWS_BIT					4
