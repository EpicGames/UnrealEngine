// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Definitions.usf: Defines undefined defines to 0 as Cg can't handle them.
=============================================================================*/

#pragma once

#ifndef MATERIAL_TWOSIDED
#define	MATERIAL_TWOSIDED								0
#endif

#ifndef MATERIAL_ISTHINSURFACE
#define	MATERIAL_ISTHINSURFACE							0
#endif

#ifndef MATERIAL_TANGENTSPACENORMAL
#define	MATERIAL_TANGENTSPACENORMAL						0
#endif

#ifndef MATERIALBLENDING_SOLID
#define MATERIALBLENDING_SOLID							0
#endif

#ifndef MATERIALBLENDING_MASKED
#define MATERIALBLENDING_MASKED							0
#endif

#ifndef MATERIALBLENDING_TRANSLUCENT
#define MATERIALBLENDING_TRANSLUCENT					0
#endif

#ifndef TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY
#define TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY			0
#endif

#ifndef MATERIALBLENDING_ADDITIVE
#define MATERIALBLENDING_ADDITIVE						0
#endif

#ifndef MATERIALBLENDING_MODULATE
#define MATERIALBLENDING_MODULATE						0
#endif

#ifndef MATERIALBLENDING_ALPHACOMPOSITE
#define MATERIALBLENDING_ALPHACOMPOSITE					0
#endif

#ifndef MATERIALBLENDING_ALPHAHOLDOUT
#define MATERIALBLENDING_ALPHAHOLDOUT					0
#endif

#ifndef MATERIAL_IS_SUBSTRATE
#define MATERIAL_IS_SUBSTRATE								0
#endif

#ifndef SUBSTRATE_BLENDING_OPAQUE
#define SUBSTRATE_BLENDING_OPAQUE							0
#endif

#ifndef SUBSTRATE_BLENDING_MASKED
#define SUBSTRATE_BLENDING_MASKED							0
#endif

#ifndef SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE
#define SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE	0
#endif

#ifndef SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE
#define SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE 0
#endif

#ifndef SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
#define SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY		0
#endif

#ifndef SUBSTRATE_BLENDING_ALPHAHOLDOUT
#define SUBSTRATE_BLENDING_ALPHAHOLDOUT					0
#endif

// We detect the isolated unlit BSDF node case to make sure lots of code is optimised out.
// When unlit is selected as part of the legacy dynamic shading model, the SLAB model is used instead.
#ifndef SUBSTRATE_OPTIMIZED_UNLIT
#define SUBSTRATE_OPTIMIZED_UNLIT							0
#endif

#ifndef SUBSTRATE_MATERIAL_ROUGHNESS_TRACKING
#define SUBSTRATE_MATERIAL_ROUGHNESS_TRACKING				1
#endif

#ifndef SUBSTRATE_NORMAL_QUALITY
#define SUBSTRATE_NORMAL_QUALITY							0
#define SUBSTRATE_TOP_LAYER_TYPE                            uint
#endif

#ifndef SUBSTRATE_LEGACY_IRIS_NORMAL
#define SUBSTRATE_LEGACY_IRIS_NORMAL						0
#endif

#ifndef SUBSTRATE_LEGACY_IRIS_TANGENT
#define SUBSTRATE_LEGACY_IRIS_TANGENT						0
#endif

#ifndef MATERIAL_SHADINGMODEL_DEFAULT_LIT
#define MATERIAL_SHADINGMODEL_DEFAULT_LIT				0
#endif

#ifndef MATERIAL_SHADINGMODEL_SUBSURFACE
#define MATERIAL_SHADINGMODEL_SUBSURFACE				0
#endif

#ifndef MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
#define MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN		0
#endif

#ifndef MATERIAL_SHADINGMODEL_UNLIT
#define	MATERIAL_SHADINGMODEL_UNLIT						0
#endif

#ifndef MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#define MATERIAL_SHADINGMODEL_SINGLELAYERWATER			0 
#endif

#ifndef SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT
#define SINGLE_LAYER_WATER_SEPARATED_MAIN_LIGHT			0 
#endif

#ifndef SINGLE_LAYER_WATER_NO_DISCARD
#define SINGLE_LAYER_WATER_NO_DISCARD					0
#endif

#ifndef MATERIAL_SINGLE_SHADINGMODEL
#define	MATERIAL_SINGLE_SHADINGMODEL					0
#endif

#ifndef HAS_PRIMITIVE_UNIFORM_BUFFER
#define	HAS_PRIMITIVE_UNIFORM_BUFFER					0
#endif

#ifndef NEEDS_VERTEX_FACTORY_INTERPOLATION
#define NEEDS_VERTEX_FACTORY_INTERPOLATION				0
#endif

// Nanite deferred material shading
#ifndef IS_NANITE_SHADING_PASS
#define IS_NANITE_SHADING_PASS 0
#endif

// Nanite programmable rasterizer
#ifndef IS_NANITE_RASTER_PASS
#define IS_NANITE_RASTER_PASS 0
#endif

#ifndef IS_NANITE_PASS
#define IS_NANITE_PASS (IS_NANITE_SHADING_PASS || IS_NANITE_RASTER_PASS)
#endif

// This is set by the material translator and signifies whether the material is using the DDX or DDY material graph nodes.
// In that case the material shader might want to implement this functionality with analytic derivatives if hardware derivatives are not supported.
#ifndef USES_EXPLICIT_DERIVATIVES
#define USES_EXPLICIT_DERIVATIVES 0
#endif

// Whether USE_ANALYTIC_DERIVATIVES may be enabled when USES_EXPLICIT_DERIVATIVES is true
#ifndef ALLOW_ANALYTIC_DERIVATIVES
#define ALLOW_ANALYTIC_DERIVATIVES 0
#endif

// Whether to use analytic derivatives
#ifndef USE_ANALYTIC_DERIVATIVES
#define USE_ANALYTIC_DERIVATIVES						(ALLOW_ANALYTIC_DERIVATIVES && USES_EXPLICIT_DERIVATIVES)
#endif

#ifndef GBUFFER_HAS_VELOCITY
#define GBUFFER_HAS_VELOCITY							0
#endif

#define PC_D3D											(SM6_PROFILE || SM5_PROFILE)

#ifndef NEEDS_LIGHTMAP_COORDINATE
#define NEEDS_LIGHTMAP_COORDINATE						0
#endif

#ifndef MATERIAL_SKY_ATMOSPHERE
#define	MATERIAL_SKY_ATMOSPHERE							0
#endif

#ifndef BASEPASS_SKYATMOSPHERE_AERIALPERSPECTIVE
#define	BASEPASS_SKYATMOSPHERE_AERIALPERSPECTIVE		0
#endif

#ifndef NUM_VF_PACKED_INTERPOLANTS
#define NUM_VF_PACKED_INTERPOLANTS						0
#endif

#ifndef REVERSEBITS
#define REVERSEBITS										reversebits
#endif

#ifndef INSTANCED_STEREO
#define INSTANCED_STEREO								0
#endif	

#ifndef MULTI_VIEW
#define MULTI_VIEW										0
#endif	

#ifndef CLEAR_COAT_BOTTOM_NORMAL
#define CLEAR_COAT_BOTTOM_NORMAL						0
#endif

#ifndef IRIS_NORMAL
#define IRIS_NORMAL										0
#endif

#ifndef TEX_COORD_SCALE_ANALYSIS
#define TEX_COORD_SCALE_ANALYSIS						0
#endif

#ifndef LIGHTMAP_VT_ENABLED
#define LIGHTMAP_VT_ENABLED								0
#endif

#define NOEXPRESSIONOPTIMIZATIONS						

#ifndef HAS_INVERTED_Z_BUFFER
#define HAS_INVERTED_Z_BUFFER	1
#endif

/** When defined sets the layer index for volumetric rendering in the vertex shader, rather than using a geometry shader */
#ifndef USING_VERTEX_SHADER_LAYER
#define USING_VERTEX_SHADER_LAYER	0
#endif

#ifndef DISABLE_FORWARD_LOCAL_LIGHTS
#define DISABLE_FORWARD_LOCAL_LIGHTS					0
#endif

// 0: Full quality, 1: mobile shading, sampling underwater depth from a texture 2: mobile shading, sampling underwater depth from the depth buffer
#ifndef SINGLE_LAYER_WATER_SHADING_QUALITY
#define SINGLE_LAYER_WATER_SHADING_QUALITY				0
#endif

#ifndef CLOUD_LAYER_PIXEL_SHADER
#define CLOUD_LAYER_PIXEL_SHADER						0
#endif


#ifndef SUPPORTS_ANISOTROPIC_MATERIALS
#define SUPPORTS_ANISOTROPIC_MATERIALS					0
#endif

#ifndef FORCE_MATERIAL_FLOAT_FULL_PRECISION
#define FORCE_MATERIAL_FLOAT_FULL_PRECISION             0
#endif

#ifndef SUPPORT_FIRST_PERSON_RENDERING
#define SUPPORT_FIRST_PERSON_RENDERING					1
#endif

///////////////////////////////////////////////////////////////////////////////
// Global Substrate material export related defines

#ifndef SUBSTRATE_MATERIAL_EXPORT_TYPE
#define SUBSTRATE_MATERIAL_EXPORT_TYPE					0
#endif

#ifndef SUBSTRATE_MATERIAL_EXPORT_CONTEXT
#define SUBSTRATE_MATERIAL_EXPORT_CONTEXT				0
#endif

#ifndef SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODE
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODE		0
#endif

#ifndef SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#define SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL			0
#endif

#if (SUBSTRATE_MATERIAL_EXPORT_TYPE > 0)
// When exporting any material properties or when using the blendable GBuffer, we always use the fully simplified Substrate materials.
#undef  SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#define SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL			1
#endif

// Those define usage should match the meaning of ESubstrateMaterialExport enum from MaterialCompiler.h
#define SUBSTRATE_MATERIAL_EXPORT_NONE							0
#define SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR_POST_COVERAGE		1
#define SUBSTRATE_MATERIAL_EXPORT_BASE_COLOR					2
#define SUBSTRATE_MATERIAL_EXPORT_NORMAL						3
#define SUBSTRATE_MATERIAL_EXPORT_EMISSIVE						4
#define SUBSTRATE_MATERIAL_EXPORT_SPECULAR						5
#define SUBSTRATE_MATERIAL_EXPORT_ROUGHNESS						6
#define SUBSTRATE_MATERIAL_EXPORT_ANISOTROPY					7
#define SUBSTRATE_MATERIAL_EXPORT_METALLIC						8
#define SUBSTRATE_MATERIAL_EXPORT_TRANSMITTANCE					9
#define SUBSTRATE_MATERIAL_EXPORT_OPACITY						10
#define SUBSTRATE_MATERIAL_EXPORT_OPACITYMASK					11
#define SUBSTRATE_MATERIAL_EXPORT_CUSTOMDATA0					12
#define SUBSTRATE_MATERIAL_EXPORT_CUSTOMDATA1					13
#define SUBSTRATE_MATERIAL_EXPORT_SUBSURFACECOLOR				14
#define SUBSTRATE_MATERIAL_EXPORT_TANGENT						15
#define SUBSTRATE_MATERIAL_EXPORT_SHADINGMODEL					16
#define SUBSTRATE_MATERIAL_EXPORT_MATERIAL_PREVIEW				17

#define SUBSTRATE_MATERIAL_EXPORT_CONTEXT_OPAQUE				0
#define SUBSTRATE_MATERIAL_EXPORT_CONTEXT_TRANSLUCENT			1

#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_OPAQUE			0
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MASKED			1
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_TRANSLUCENT		2
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_ADDITIVE			3
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_MODULATE			4
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_ALPHACOMPOSITE	5
#define SUBSTRATE_MATERIAL_EXPORT_LEGACY_BLEND_ALPHAHOLDOUT		6

#define SUBSTRATE_MATERIAL_EXPORT_EXECUTED						(SUBSTRATE_MATERIAL_EXPORT_TYPE > 0)
#define SUBSTRATE_MATERIAL_EXPORT_FROM_OPAQUE					(SUBSTRATE_MATERIAL_EXPORT_EXECUTED && (SUBSTRATE_MATERIAL_EXPORT_CONTEXT == SUBSTRATE_MATERIAL_EXPORT_CONTEXT_OPAQUE))
#define SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT				(SUBSTRATE_MATERIAL_EXPORT_EXECUTED && (SUBSTRATE_MATERIAL_EXPORT_CONTEXT == SUBSTRATE_MATERIAL_EXPORT_CONTEXT_TRANSLUCENT))


