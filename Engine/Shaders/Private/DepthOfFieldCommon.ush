// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	DepthOfFieldCommon.ush: depth of field common
=============================================================================*/

// [0] .x:SkyFocusDistance or very value if 0, .y:DepthOfFieldVignetteMul, .z:DepthOfFieldVignetteAdd, w:OcclusionTweak1
float4 DepthOfFieldParams;


// Computed the "Circle Of Confusion" radius for "Depth of Field"
// Formula can be found in many places e.g. http://http.developer.nvidia.com/GPUGems/gpugems_ch23.html
// @param SceneDepth
// @return 0..1 0=in focus, 1:max blurry
float ComputeCircleOfConfusion(float SceneDepth)
{
	// artificial area where all content is in focus (starting at FocalLength, ending at FocalLength+FocalRegion)
	FLATTEN if(SceneDepth > View.DepthOfFieldFocalDistance)
	{
		SceneDepth = View.DepthOfFieldFocalDistance + max(0, SceneDepth - View.DepthOfFieldFocalDistance - View.DepthOfFieldFocalRegion);
	}

	// depth of the pixel in unreal units
	float D = SceneDepth;
	// e.g. Focal length in mm (Camera property e.g. 75mm)
	float F = View.DepthOfFieldFocalLength;
	// Plane in Focus in unreal units
	float P = View.DepthOfFieldFocalDistance;
	// Camera property e.g. 0.5f, like aperture
	float Aperture = View.DepthOfFieldScale;
		
	 
	// convert unreal units (100=1m) to mm
	P *= 0.001f / 100.0f;
	D *= 0.001f / 100.0f;
/*
	float Div = abs(D * (P - F));
	// clamp crazy numbers
//	Div = max(0.01f, Div);
	float CoCRadiusFactor = Aperture * F * abs(P - D) / Div;
	return saturate(CoCRadiusFactor);
*/
	// note: F has almost no effect
	float CoCRadius = Aperture * F * (P - D) / (D * (P - F));

	return saturate(abs(CoCRadius));
}


// @param SceneDepth
// @return 0..1 0=in focus, 1:max blurry
float ComputeCircleOfConfusionNorm(float SceneDepth)
{
	// artificial area where all content is in focus (starting at FocalLength, ending at FocalLength+FocalRegion)
	FLATTEN if(SceneDepth > View.DepthOfFieldFocalDistance)
	{
		SceneDepth = View.DepthOfFieldFocalDistance + max(0, SceneDepth - View.DepthOfFieldFocalDistance - View.DepthOfFieldFocalRegion);
	}

	//	assumed to be >0 by c++ setup code
	half TransitionRegion = (SceneDepth < View.DepthOfFieldFocalDistance) ? View.DepthOfFieldNearTransitionRegion : View.DepthOfFieldFarTransitionRegion;

	return saturate(abs(SceneDepth - View.DepthOfFieldFocalDistance) / TransitionRegion);
}

/**
 * Computes the unfocused percent for a scene depth.
 * @param SceneDepth - The scene depth.
 * @return A unfocused percent for the depth. 0..1 (min and max ensures we don't exceed the range)
 */
half CalcUnfocusedPercentCustomBound(float SceneDepth, float MaxBlurNear, float MaxBlurFar)
{
	half MaxUnfocusedPercent = (SceneDepth < View.DepthOfFieldFocalDistance) ? MaxBlurNear : MaxBlurFar;

	half Unbound = ComputeCircleOfConfusionNorm(SceneDepth);

	return min(MaxUnfocusedPercent, Unbound);
}
