// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DepthOnlyPixelShader.hlsl: Depth-only pixel shader.
=============================================================================*/

#if !MATERIAL_LWC_ENABLED
#define UE_DF_FORCE_FP32_OPS 1
#endif

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

void Main(
#if !MATERIALBLENDING_SOLID || OUTPUT_PIXEL_DEPTH_OFFSET
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants
	, float4 PixelPosition : TEXCOORD6 //-- Not used currently
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	, float3 PixelPositionExcludingWPO : TEXCOORD7
#endif
	, in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position
	, in FStereoPSInput StereoInput
	OPTIONAL_IsFrontFace
	OPTIONAL_OutDepthConservative,
#endif
	out float4 OutColor : SV_Target0
#if MATERIALBLENDING_MASKED_USING_COVERAGE
	, out uint OutCoverage : SV_Coverage
#endif
	)
{
#if !MATERIALBLENDING_SOLID || OUTPUT_PIXEL_DEPTH_OFFSET
	StereoSetupPS(StereoInput);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;

	#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, PixelPositionExcludingWPO);	
	#else
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
	#endif

	#if OUTPUT_PIXEL_DEPTH_OFFSET
		#if ALLOW_DEBUG_VIEW_MODES
		OutDepth = SvPosition.z;
		if (!ResolvedView.DebugViewModeMask)
		#endif
		{
			ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);
		}

		#if APPLE_DEPTH_BIAS_HACK
		OutDepth -= APPLE_DEPTH_BIAS_VALUE;
		#endif
	#endif





	#if SUBSTRATE_GBUFFER_FORMAT==1
		FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
		FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

		float MaterialOpacity = 1.0f;
		if (SubstratePixelHeader.ClosureCount > 0)
		{
			// Update tree (coverage/transmittance/luminace weights)
			const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();
			const float3 V = MaterialParameters.CameraVector;
			SubstratePixelHeader.SubstrateUpdateTree(V, Settings);

			MaterialOpacity = SubstratePixelHeader.SubstrateTree.Operators[SubstrateData.OperatorIndex].Coverage;
		}

	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		MaterialOpacity = GetMaterialOpacity(PixelMaterialInputs); 
	#endif

	#else // SUBSTRATE_GBUFFER_FORMAT==1

		float MaterialOpacity = GetMaterialOpacity(PixelMaterialInputs);

	#endif // SUBSTRATE_GBUFFER_FORMAT==1
		

	#if MATERIALBLENDING_TRANSLUCENT
		clip(MaterialOpacity - GetMaterialOpacityMaskClipValue());
	#elif MATERIALBLENDING_MASKED_USING_COVERAGE
		OutCoverage = DiscardMaterialWithPixelCoverage(MaterialParameters, PixelMaterialInputs);
	#else
		GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);
	#endif
#endif

	OutColor = 0;
}
