// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFBokehLUT.usf: Generate LUT according to bokeh shape.
=============================================================================*/

#include "DOFCommon.ush"


//------------------------------------------------------- ENUM VALUES.

// Matching C++'s EDiaphragmDOFGatherRecursionMode
#define BOKEH_SHAPE_CIRCLE 0
#define BOKEH_SHAPE_STRAIGHT_BLADES 1
#define BOKEH_SHAPE_ROUNDED_BLADES 2


#define LUT_FORMAT_DISTANCE_FACTOR 0
#define LUT_FORMAT_FULL_RES_TO_COC_DISTANCE 1
#define LUT_FORMAT_GATHER_SAMPLE_POS 2


//------------------------------------------------------- CONFIG

#if DIM_ROUND_BLADES
	#define BOKEH_SHAPE (BOKEH_SHAPE_ROUNDED_BLADES)

#else
	#define BOKEH_SHAPE (BOKEH_SHAPE_STRAIGHT_BLADES)

#endif

#if DIM_LUT_FORMAT == LUT_FORMAT_DISTANCE_FACTOR
	#define CONFIG_CENTER_BOKEH_AT_UV_ORIGIN 1

#elif DIM_LUT_FORMAT == LUT_FORMAT_FULL_RES_TO_COC_DISTANCE
#endif


// Whether the center of the boked should be positioned at UV=(0, 0)
#ifndef CONFIG_CENTER_BOKEH_AT_UV_ORIGIN
	#define CONFIG_CENTER_BOKEH_AT_UV_ORIGIN 0
#endif


//------------------------------------------------------- COMPILE TIME CONSTANTS

#define THREADGROUP_SIZEX		(8)
#define THREADGROUP_SIZEY		(THREADGROUP_SIZEX)
#define THREADGROUP_TOTALSIZE	(THREADGROUP_SIZEX * THREADGROUP_SIZEY)


//------------------------------------------------------- PARAMETERS

// Number of blades >= 4.
uint BladeCount;

// Factor to convert a coc radius to bokeh's circomscribed & incircle radius
float CocRadiusToCircumscribedRadius;
float CocRadiusToIncircleRadius;

// Rotation of the blades of the diaphragm.
float DiaphragmRotation;

// Radius of the blade for a boked area=PI.
float DiaphragmBladeRadius;

// Offset.
float DiaphragmBladeCenterOffset;


//------------------------------------------------------- FUNCTIONS

/** Compute scaling factor that needs to be applied to radial coordinate (R=1, PixelAngle)'s radius R,
 * to have the point projected onto the bokeh's edge with the bokeh area remaining equal to PI.
 */
float ComputeCocRadiusToBokehEdgeFactor(float PixelAngle)
{
	float EdgeId = floor(PixelAngle * rcp(2 * PI) * float(BladeCount));
	float EdgeAngle = (EdgeId + 0.5) * (2 * PI) * rcp(float(BladeCount));
	
	float Alpha = PixelAngle - EdgeAngle;

	float CocRadiusToBokehEdgeFactor;
	if (BOKEH_SHAPE == BOKEH_SHAPE_CIRCLE)
	{
		// Pointless, but here just for testing purpose.
		CocRadiusToBokehEdgeFactor = 1;
	}
	else if (BOKEH_SHAPE == BOKEH_SHAPE_STRAIGHT_BLADES)
	{
		CocRadiusToBokehEdgeFactor = CocRadiusToIncircleRadius / cos(Alpha);
	}
	else if (BOKEH_SHAPE == BOKEH_SHAPE_ROUNDED_BLADES)
	{
		/**
		 * Resolve t:
		 *  (x + d) ^ 2 + y ^ 2 = r ^ 2
		 *  x = t cos(alpha)
		 *  y = t sin(alpha)
		 *
		 *  => t^2 + t * (2 * d * cos(alpha) + d^2 - r^2 = 0
		 */
		const float a = 1;
		float b = 2 * DiaphragmBladeCenterOffset * cos(Alpha);
		float c = DiaphragmBladeCenterOffset * DiaphragmBladeCenterOffset - DiaphragmBladeRadius * DiaphragmBladeRadius;

		float Delta = b * b - 4 * a * c;

		/**
		 * Only want positive solution that we know exists because:
		 *  a > 0, b > 0 and c < 0 => Delta > b^2 => t > 0.
		 */
		float t = (-b + sqrt(Delta)) * rcp(2 * a);

		CocRadiusToBokehEdgeFactor = t;
	}

	return CocRadiusToBokehEdgeFactor;
}


void RingCoordinateFromCartesianCoordinate(const float2 C, out float OutRingRadius, out float OutRingSampleId)
{
	float2 DialC = C;
	float DialId = 0;

	UNROLL
	for (uint i = 0; i < 3; i++)
	{
		FLATTEN
		if (DialC.x < 0.0 || DialC.y < 0.0)
		{
			DialC = float2(DialC.y, -DialC.x);
			DialId += 1;
		}
	}
	
	OutRingRadius = max(abs(DialC.x), abs(DialC.y));
	OutRingSampleId = DialId * 2 * OutRingRadius;

	FLATTEN
	if (DialC.x > DialC.y)
	{
		OutRingSampleId += DialC.y;
	}
	else
	{
		OutRingSampleId += 2 * OutRingRadius - DialC.x;
	}
}


//------------------------------------------------------- OUTPUTS

RWTexture2D<float4>	BokehLUTOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void BuildBokehLUTMainCS(uint2 DispatchThreadId : SV_DispatchThreadID) 
{
	float2 RelativePosFromCenter;

	#if CONFIG_CENTER_BOKEH_AT_UV_ORIGIN
	{
		// Position center at UV=(0, 0)
		RelativePosFromCenter = float2(DispatchThreadId) + 0.5;

		if (RelativePosFromCenter.x > BOKEH_LUT_SIZE / 2)
		{
			RelativePosFromCenter.x = RelativePosFromCenter.x - BOKEH_LUT_SIZE;
		}
		if (RelativePosFromCenter.y > BOKEH_LUT_SIZE / 2)
		{
			RelativePosFromCenter.y = RelativePosFromCenter.y - BOKEH_LUT_SIZE;
		}
	}
	#else
	{
		// Position center at UV=(1, 1) * (0.5 + 0.5 / BOKEH_LUT_SIZE)
		RelativePosFromCenter = float2(DispatchThreadId) - float(BOKEH_LUT_SIZE / 2);
	}
	#endif

	#if DIM_LUT_FORMAT == LUT_FORMAT_DISTANCE_FACTOR
	{
		float PixelAngle = atan2(RelativePosFromCenter.y, RelativePosFromCenter.x) + DiaphragmRotation + PI;

		float CocRadiusToBokehEdgeFactor = ComputeCocRadiusToBokehEdgeFactor(PixelAngle);

		BokehLUTOutput[DispatchThreadId] = float4(CocRadiusToBokehEdgeFactor, 0, 0, 0);
	}
	#elif DIM_LUT_FORMAT == LUT_FORMAT_FULL_RES_TO_COC_DISTANCE
	{
		float PixelAngle = atan2(RelativePosFromCenter.y, RelativePosFromCenter.x) + DiaphragmRotation + PI;

		float CocRadiusToBokehEdgeFactor = ComputeCocRadiusToBokehEdgeFactor(PixelAngle);
		if (all(RelativePosFromCenter == 0.0))
		{
			CocRadiusToBokehEdgeFactor = 1.0;
		}

		float PixelCocDistanceFromBokehCenter = FullResPixelDistanceToCocDistance(length(RelativePosFromCenter * float2(CocSqueeze, 1.0)) / CocRadiusToBokehEdgeFactor);
		BokehLUTOutput[DispatchThreadId] = float4(PixelCocDistanceFromBokehCenter, 0, 0, 0);
	}
	#elif DIM_LUT_FORMAT == LUT_FORMAT_GATHER_SAMPLE_POS
	{
		float InvBladeCount = rcp(float(BladeCount));

		float RingRadius;
		float RingSampleId;
		RingCoordinateFromCartesianCoordinate(RelativePosFromCenter, RingRadius, RingSampleId);

		// Number of sample on the ring.
		float RingSampleCount = max(1, RingRadius * (4 * 2));

		// TODO: as the bokeh rotate, interferances are introduced in gather pass that post filtering just can't keep up with.
		// Need to find a proper fix.
		float PixelAngle = ((RingSampleId + frac(RingRadius * 0.5)) / RingSampleCount) * (2 * PI) + PI;
		
		float EdgeId = floor(PixelAngle * rcp(2 * PI) * float(BladeCount));
		float EdgeInterp = frac(PixelAngle * rcp(2 * PI) * float(BladeCount));

		// Compute the position of the sample on blade edge (assuming kernel area = PI).
		float2 KernelUnitPos;
		if (BOKEH_SHAPE == BOKEH_SHAPE_CIRCLE)
		{
			// Pointless, but here just for testing purpose.
			KernelUnitPos = float2(cos(PixelAngle), sin(PixelAngle));
		}
		else if (BOKEH_SHAPE == BOKEH_SHAPE_STRAIGHT_BLADES)
		{
			float A0 = (EdgeId + 0.0) * ((2 * PI) * InvBladeCount) - DiaphragmRotation;
			float A1 = A0 + ((2 * PI) * InvBladeCount);

			float2 P0 = float2(cos(A0), sin(A0));
			float2 P1 = float2(cos(A1), sin(A1));
			KernelUnitPos = CocRadiusToCircumscribedRadius * lerp(P0, P1, EdgeInterp); // TODO.
		}
		else if (BOKEH_SHAPE == BOKEH_SHAPE_ROUNDED_BLADES)
		{
			// Compute the position of the sample on blade edge (assuming kernel area = PI).
			float BetaAngle = asin(CocRadiusToCircumscribedRadius / DiaphragmBladeRadius * sin(PI / float(BladeCount)));
			float SampleBladeAngle = BetaAngle * (EdgeInterp * 2 - 1);

			float2 BladeUnitPos;
			BladeUnitPos.x = DiaphragmBladeRadius * cos(SampleBladeAngle) - DiaphragmBladeCenterOffset;
			BladeUnitPos.y = DiaphragmBladeRadius * sin(SampleBladeAngle);
			
			// Transform from blade space -> kernel space.
			float BladeAngle = (EdgeId + 0.5) * (2 * PI) * InvBladeCount - DiaphragmRotation;
			KernelUnitPos = float2(
				BladeUnitPos.x * cos(BladeAngle) - BladeUnitPos.y * sin(BladeAngle),
				BladeUnitPos.x * sin(BladeAngle) + BladeUnitPos.y * cos(BladeAngle));
		}

		// Final position of the sample in kernel.
		float2 SamplePos = KernelUnitPos * RingRadius;
		
		// Apply anamorphic squeeze factor to save VALU in DOFGatherKernel.ush
		SamplePos.x *= CocInvSqueeze;

		BokehLUTOutput[DispatchThreadId] = float4(SamplePos, 0, 0);
	}
	#else
		#error Unknown LUT format.
	#endif
}
