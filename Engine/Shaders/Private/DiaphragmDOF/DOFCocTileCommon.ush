// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFCocTileCommon.ush: Common code to read and write coc tile buffer.
=============================================================================*/

#pragma once

#include "DOFCommon.ush"


//------------------------------------------------------- COMPILE TIME CONSTANTS.

// Maximum number of buffer for Coc tile.
#define COC_TILE_MAX_BUFFER_COUNT 2


//------------------------------------------------------- ENUM VALUES

/** Different layout of CocTile buffers. */

// 1 foreground encoded in RG channels, and 1 background encoded in BA channels.
#define COC_TILE_LAYOUT_FGD_BGD 0

// 1 foreground encoded in RG channels. Use same as COC_TILE_LAYOUT_FGD_BGD because this would
// be used fprergound only pass, which means all background code would be compiled out.
#define COC_TILE_LAYOUT_FGD (COC_TILE_LAYOUT_FGD_BGD)

// 1 foreground encoded in RG channels if RT 0, 1 background encoded in RGB channels if RT 1.
#define COC_TILE_LAYOUT_FGD_SEP_BGD 1


/** Returns the number of buffers for a Coc tile layout. */
uint GetBufferCountForCocTileLayour(uint Layout)
{
	if (Layout == COC_TILE_LAYOUT_FGD_SEP_BGD)
	{
		return 2;
	}
	return 1;
}


//------------------------------------------------------- STRUCTS

/** Tile buffer. */
struct FCocTileSample
{
	// The minimum Coc for foreground <= 0.
	float FgdMinCoc;

	// The maximum Coc for forground <= 0.
	float FgdMaxCoc;

	// The minimum Coc for background >= 0.
	float BgdMinCoc;

	// The maximum Coc for background >= 0.
	float BgdMaxCoc;

	// The minimum intersectable Coc radius >= 0.
	float BgdMinIntersectableCoc;

	// The conservative distance field to any closer geometry.
	float BgdMinConservativeCloserGoemetryDistance;
};


//------------------------------------------------------- Encoding/decoding.

void InitCocTileSample(out FCocTileSample Sample)
{
	Sample.FgdMinCoc = 0;
	Sample.FgdMaxCoc = 0;
	Sample.BgdMinCoc = 0;
	Sample.BgdMaxCoc = 0;
	Sample.BgdMinIntersectableCoc = EXTREMELY_LARGE_COC_RADIUS;
	Sample.BgdMinConservativeCloserGoemetryDistance = 0;
}

/** Encode coc tile sample.
 * @param <OutputLayout>: the layout of the coc buffer to output.
 */
void EncodeCocTileSample(in FCocTileSample Sample, in uint Layout, out float4 Raw[COC_TILE_MAX_BUFFER_COUNT])
{
	for (uint i = 0; i < COC_TILE_MAX_BUFFER_COUNT; i++)
	{
		Raw[i] = 0;
	}
	
	if (static_condition(Layout == COC_TILE_LAYOUT_FGD_BGD))
	{
		Raw[0].rg = float2(Sample.FgdMinCoc, Sample.FgdMaxCoc);
		Raw[0].ba = float2(Sample.BgdMinCoc, Sample.BgdMaxCoc);
	}
	else if (static_condition(Layout == COC_TILE_LAYOUT_FGD_SEP_BGD))
	{
		Raw[0].rg = float2(Sample.FgdMinCoc, Sample.FgdMaxCoc);
		Raw[1] = float4(Sample.BgdMaxCoc, Sample.BgdMinCoc, Sample.BgdMinIntersectableCoc, Sample.BgdMinConservativeCloserGoemetryDistance);
	}
}

/** Decode coc tile sample.
 * @param <InputLayout>: the layout of the coc buffer.
 */
void DecodeCocTileSample(in float4 Raw[COC_TILE_MAX_BUFFER_COUNT], in uint Layout, out FCocTileSample Sample)
{
	InitCocTileSample(Sample);

	if (static_condition(Layout == COC_TILE_LAYOUT_FGD_BGD))
	{
		Sample.FgdMinCoc = Raw[0].r;
		Sample.FgdMaxCoc = Raw[0].g;
		Sample.BgdMinCoc = Raw[0].b;
		Sample.BgdMaxCoc = Raw[0].a;
	}
	else if (static_condition(Layout == COC_TILE_LAYOUT_FGD_SEP_BGD))
	{
		Sample.FgdMinCoc = Raw[0].r;
		Sample.FgdMaxCoc = Raw[0].g;
		Sample.BgdMinCoc = Raw[1].g;
		Sample.BgdMaxCoc = Raw[1].r;
		Sample.BgdMinIntersectableCoc = Raw[1].b;
		Sample.BgdMinConservativeCloserGoemetryDistance = Raw[1].a;
	}
}



//------------------------------------------------------- Higher level API.

/** Sample and decode tile buffer. */
FCocTileSample LoadCocTile(const uint Layout, Texture2D Buffer0, Texture2D Buffer1, int2 TilePos)
{
	float4 PackedSample[COC_TILE_MAX_BUFFER_COUNT];
	PackedSample[0] = Buffer0[TilePos];
	PackedSample[1] = Buffer1[TilePos];

	FCocTileSample UnpackedSample;
	DecodeCocTileSample(PackedSample, Layout, UnpackedSample);
	return UnpackedSample;
}
