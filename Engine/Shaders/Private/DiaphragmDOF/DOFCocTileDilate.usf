// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFCocDilate.usf: Dilate the coc tile buffer.
=============================================================================*/

#include "DOFCommon.ush"
#include "DOFCocTileCommon.ush"


//------------------------------------------------------- ENUM VALUES

/** Matches FRCPassDiaphragmDOFDilateCoc::EMode. */
	#define DILATE_MODE_STANDALONE 0
	#define DILATE_MODE_MIN_MAX 1
	#define DILATE_MODE_MIN_ABS 2


//------------------------------------------------------- PASSES.

uint4 ViewportRect;
uint SampleOffsetMultipler;
float fSampleOffsetMultipler;

// Multiplier to compare CocRadius and bucket distance.
float CocRadiusToBucketDistanceUpperBound;
float BucketDistanceToCocRadius;

Texture2D TileInput_Foreground;
Texture2D TileInput_Background;

Texture2D DilatedTileMinMax_Foreground;
Texture2D DilatedTileMinMax_Background;


//------------------------------------------------------- OUTPUTS.

RWTexture2D<float4>	TileOutput_Foreground;
RWTexture2D<float4>	TileOutput_Background;


//------------------------------------------------------- ENTRY POINT.

// TODO: could support larger radius with LDS.
[numthreads(DEFAULT_GROUP_BORDER_SIZE, DEFAULT_GROUP_BORDER_SIZE, 1)]
void CocDilateMainCS(uint2 DispatchThreadId : SV_DispatchThreadID) 
{
	ResolvedView = ResolveView();

	const uint CocTileLayout = COC_TILE_LAYOUT_FGD_SEP_BGD;
	const uint Radius = DIM_DILATE_RADIUS;

	const uint2 PixelPos = DispatchThreadId;

	// Different buckets of CocRadius per gathering radius.
	FCocTileSample OutSampleBuckets[Radius];

	// Standalone does not have any radius multiplier since there is only one dilate pass.
	#if DIM_DILATE_MODE == DILATE_MODE_STANDALONE
		const uint RadiusMultiplier = 1;
		const float fRadiusMultiplier = 1;
	#else
		const uint RadiusMultiplier = SampleOffsetMultipler;
		const float fRadiusMultiplier = fSampleOffsetMultipler;
	#endif

	// Fetches the already dilated min foreground and max background COC radius.
	#if DIM_DILATE_MODE == DILATE_MODE_MIN_ABS
		FCocTileSample DilatedMinMaxSample = LoadCocTile(CocTileLayout, DilatedTileMinMax_Foreground, DilatedTileMinMax_Background, PixelPos);
	#endif

	// Iterate over all input tiles and assign them into their buckets.
	UNROLL
	for (uint BucketId = 0; BucketId < Radius; BucketId++)
	{
		uint RingDistance = BucketId + 1;
		
		// Init the bucket.
		OutSampleBuckets[BucketId].FgdMinCoc = 0;
		OutSampleBuckets[BucketId].FgdMaxCoc = -EXTREMELY_LARGE_COC_RADIUS;
		OutSampleBuckets[BucketId].BgdMinCoc = EXTREMELY_LARGE_COC_RADIUS;
		OutSampleBuckets[BucketId].BgdMaxCoc = 0;
		OutSampleBuckets[BucketId].BgdMinIntersectableCoc = EXTREMELY_LARGE_COC_RADIUS;
		OutSampleBuckets[BucketId].BgdMinConservativeCloserGoemetryDistance = EXTREMELY_LARGE_COC_RADIUS;

		UNROLL_N(DIM_DILATE_RADIUS*4)
		for (uint BatchId = 0; BatchId < 4 * RingDistance; BatchId++)
		{
			// Generates at compile time sample location if square shaped ring in sample count distance from the kernel's center.
			//
			// RingId = 0:
			//      Y
			//      |
			//    3 2 1
			//    4   0 -X
			//
			// RingId = 1:
			//      Y
			//      |
			//  6 5 4 3 2
			//  7       1
			//  8       0 -X
			//
			// ...
			int2 BatchSquarePos;
			if (BatchId < RingDistance)
			{
				BatchSquarePos.x = int(RingDistance);
				BatchSquarePos.y = int(BatchId);
			}
			else if (BatchId < RingDistance * 3)
			{
				BatchSquarePos.x = int(RingDistance) - int(BatchId - RingDistance);
				BatchSquarePos.y = int(RingDistance);
			}
			else
			{
				BatchSquarePos.x = -int(RingDistance);
				BatchSquarePos.y = int(RingDistance) - (int(BatchId) - int(RingDistance) * 3);
			}

			UNROLL
			for (uint SampleId = 0; SampleId < 2; SampleId++)
			{
				const int2 Offset = (SampleId == 0 ? 1 : -1) * (int(RadiusMultiplier) * BatchSquarePos);
				int2 SampleIndex = int2(PixelPos) + Offset;
			
				// If sampling outside the tiles, ignore this sample.
				bool2 bInsideViewport = uint2(SampleIndex) < uint2(ViewportRect.zw);
				if (all(bInsideViewport))
				{
					// Minimal CoC radius to consider because done at full res.
					const float MinimalCoC = 0.5 * kCocRadiusToFullResFactor;

					// Closest distance from sampling tile to output tile.
					float ClosestDistance = length(max(fRadiusMultiplier * (COC_TILE_SIZE * abs(float2(Offset))) - COC_TILE_SIZE, 0.0));
					//ClosestDistance = fRadiusMultiplier * float((BucketId + 1) * COC_TILE_SIZE) - COC_TILE_SIZE;

					// Sample tile.
					const FCocTileSample NeightborSample = LoadCocTile(CocTileLayout, TileInput_Foreground, TileInput_Background, SampleIndex);

					OutSampleBuckets[BucketId].FgdMinCoc = min(
						OutSampleBuckets[BucketId].FgdMinCoc,
						NeightborSample.FgdMinCoc);

					OutSampleBuckets[BucketId].FgdMaxCoc = max(
						OutSampleBuckets[BucketId].FgdMaxCoc,
						NeightborSample.FgdMaxCoc);

					#if DIM_DILATE_MODE == DILATE_MODE_MIN_ABS && 0 // TODO.
						if (NeightborSample.BgdMinConservativeCloserGoemetryDistance + ClosestDistance * BucketDistanceToCocRadius < 2 * DilatedMinMaxSample.BgdMaxCoc)
					#endif
					{
						OutSampleBuckets[BucketId].BgdMinCoc = min(
							OutSampleBuckets[BucketId].BgdMinCoc,
							NeightborSample.BgdMinCoc);

						OutSampleBuckets[BucketId].BgdMaxCoc = max(
							OutSampleBuckets[BucketId].BgdMaxCoc,
							NeightborSample.BgdMaxCoc);
					
						OutSampleBuckets[BucketId].BgdMinIntersectableCoc = min(
							OutSampleBuckets[BucketId].BgdMinIntersectableCoc,
							NeightborSample.BgdMinIntersectableCoc + ClosestDistance * BucketDistanceToCocRadius);
					}
							
					OutSampleBuckets[BucketId].BgdMinConservativeCloserGoemetryDistance = min(
						OutSampleBuckets[BucketId].BgdMinConservativeCloserGoemetryDistance,
						NeightborSample.BgdMinConservativeCloserGoemetryDistance + ClosestDistance * BucketDistanceToCocRadius);
				}
			} // for (uint SampleId = 0; SampleId < 2; SampleId++)
		} // for (uint BatchId = 0; BatchId < 4 * RingDistance; BatchId++)
	} // for (uint BucketId = 0; BucketId < Radius; BucketId++)
	
	// Sample the center bucket.
	FCocTileSample OutSample = LoadCocTile(CocTileLayout, TileInput_Foreground, TileInput_Background, PixelPos);

	#if DIM_DILATE_MODE == DILATE_MODE_MIN_ABS
	{
		OutSample.FgdMinCoc = DilatedMinMaxSample.FgdMinCoc;
		OutSample.BgdMaxCoc = DilatedMinMaxSample.BgdMaxCoc;
	}
	#else // DIM_DILATE_MODE != DILATE_MODE_MIN_ABS
	{
		// Find out what the maximum absolute Coc radii will be for the output sample.
		UNROLL
		for (uint k = 0; k < Radius; k++)
		{
			float BucketDistance = fRadiusMultiplier * (float(k + 1) * COC_TILE_SIZE) - COC_TILE_SIZE;

			FLATTEN
			if (-OutSampleBuckets[k].FgdMinCoc * CocRadiusToBucketDistanceUpperBound > BucketDistance)
			{
				OutSample.FgdMinCoc = min(OutSample.FgdMinCoc, OutSampleBuckets[k].FgdMinCoc);
			}
			
			FLATTEN
			if (OutSampleBuckets[k].BgdMaxCoc * CocRadiusToBucketDistanceUpperBound > BucketDistance)
			{
				OutSample.BgdMaxCoc = max(OutSample.BgdMaxCoc, OutSampleBuckets[k].BgdMaxCoc);
			}
		}
	}
	#endif
		
	// Then, find out what is minimum absolute Coc radii that will be intersected for these maximum Coc radii.
	{
		const float CocRadiusErrorFraction = 0.05;
		float ForegroundCocRadiusError = -OutSample.FgdMinCoc * CocRadiusErrorFraction;
		float BackgroundCocRadiusError = OutSample.BgdMaxCoc * CocRadiusErrorFraction;

		UNROLL
		for (uint k = 0; k < Radius; k++)
		{
			float BucketDistance = fRadiusMultiplier * (float(k + 1) * COC_TILE_SIZE) - COC_TILE_SIZE;
			
			const float MinimalCoC = 0.5 * kCocRadiusToFullResFactor;

			FLATTEN
			if (-OutSample.FgdMinCoc * CocRadiusToBucketDistanceUpperBound > BucketDistance)
			{
				OutSample.FgdMaxCoc = max(OutSample.FgdMaxCoc, OutSampleBuckets[k].FgdMaxCoc);
			}

			FLATTEN
			if (OutSample.BgdMaxCoc * CocRadiusToBucketDistanceUpperBound > BucketDistance)
			{
				OutSample.BgdMinCoc = min(OutSample.BgdMinCoc, OutSampleBuckets[k].BgdMinCoc);
				OutSample.BgdMinIntersectableCoc = min(OutSample.BgdMinIntersectableCoc, OutSampleBuckets[k].BgdMinIntersectableCoc);
			}

			OutSample.BgdMinConservativeCloserGoemetryDistance = min(OutSample.BgdMinConservativeCloserGoemetryDistance, OutSampleBuckets[k].BgdMinConservativeCloserGoemetryDistance);
		}
	}

	// If dilating min foreground and max background coc, zero output other to prune the unecessary ALU.
	#if DIM_DILATE_MODE == DILATE_MODE_MIN_MAX
	{
		OutSample.FgdMaxCoc = 0;
		OutSample.BgdMinCoc = 0;
		OutSample.BgdMinIntersectableCoc = 0;
		OutSample.BgdMinConservativeCloserGoemetryDistance = 0;
	}
	#endif

	// Output that.
	if (all(PixelPos < ViewportRect.zw))
	{
		float4 RawOutput[COC_TILE_MAX_BUFFER_COUNT];
		EncodeCocTileSample(OutSample, CocTileLayout, RawOutput);

		TileOutput_Foreground[PixelPos] = RawOutput[0];
		if (static_condition(GetBufferCountForCocTileLayour(CocTileLayout) > 1))
		{
			TileOutput_Background[PixelPos] = RawOutput[1];
		}
	}
}
