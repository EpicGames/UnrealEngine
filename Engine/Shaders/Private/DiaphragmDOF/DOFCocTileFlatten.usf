// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFCocFlatten.usf: Flatten Coc radius into coc tile buffer.
=============================================================================*/

#include "DOFCommon.ush"
#include "DOFCocTileCommon.ush"


//------------------------------------------------------- COMPILE TIME CONFIG

// Method used to flatten CocRadius.
	#define FLATTEN_METHOD_REDUCE_LDS 0
	#define FLATTEN_METHOD_ATOMIC_LDS 1
	#define FLATTEN_METHOD_WAVE_MINMAX 2

#if COMPILER_SUPPORTS_WAVE_MINMAX && COMPILER_SUPPORTS_WAVE_ONCE
	#define CONFIG_FLATTEN_METHOD (FLATTEN_METHOD_WAVE_MINMAX)
#else
	#define CONFIG_FLATTEN_METHOD (FLATTEN_METHOD_REDUCE_LDS)
#endif


//------------------------------------------------------- PARAMETERS.

float2 Stride;
float4 GatherInputExtentAndInvExtent;
float4 GatherInputViewRectUV;

Texture2D GatherInput_SceneColor;
Texture2D GatherInput_SeparateCoc;


//------------------------------------------------------- OUTPUTS.

RWTexture2D<float4>	TileOutput_Foreground;
RWTexture2D<float4>	TileOutput_Background;


//------------------------------------------------------- LDS.

#define LDS_LAYOUT (COC_TILE_LAYOUT_FGD_SEP_BGD)


void InitCocTileSampleFromCocRadius(float CocRadius, out FCocTileSample OutA)
{
	InitCocTileSample(OutA);

	OutA.FgdMinCoc = min(CocRadius, 0);
	OutA.FgdMaxCoc = OutA.FgdMinCoc;
				
	OutA.BgdMinCoc = max(CocRadius, 0);
	OutA.BgdMaxCoc = OutA.BgdMinCoc;
	OutA.BgdMinIntersectableCoc = (CocRadius > 0) ? CocRadius : EXTREMELY_LARGE_COC_RADIUS;
}

void ReduceOperator(inout FCocTileSample A, in const FCocTileSample B)
{
	A.FgdMinCoc = min(A.FgdMinCoc, B.FgdMinCoc);
	A.FgdMaxCoc = max(A.FgdMaxCoc, B.FgdMaxCoc);

	A.BgdMinCoc = min(A.BgdMinCoc, B.BgdMinCoc);
	A.BgdMaxCoc = max(A.BgdMaxCoc, B.BgdMaxCoc);
	A.BgdMinIntersectableCoc = min(A.BgdMinIntersectableCoc, B.BgdMinIntersectableCoc);
}


#if CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_REDUCE_LDS

// TODO: could technically only need half of it: would this be faster?
groupshared float4 Shared0[DEFAULT_GROUP_BORDER_SIZE * DEFAULT_GROUP_BORDER_SIZE];
groupshared float Shared1[DEFAULT_GROUP_BORDER_SIZE * DEFAULT_GROUP_BORDER_SIZE];

FCocTileSample LoadFromLds(uint i)
{
	float4 Raw[2];
	Raw[0] = Shared0[i];
	Raw[1] = 0;
	
	if (LDS_LAYOUT == COC_TILE_LAYOUT_FGD_SEP_BGD)
	{
		Raw[0] = Shared0[i];
		Raw[1].rg = Shared0[i].ba;
		Raw[1].b = Shared1[i];
	}
	
	FCocTileSample Sample;
	DecodeCocTileSample(Raw, LDS_LAYOUT, Sample);
	return Sample;
}

void StoreToLds(uint i, in const FCocTileSample Sample)
{
	float4 Raw[COC_TILE_MAX_BUFFER_COUNT];
	EncodeCocTileSample(Sample, LDS_LAYOUT, Raw);
	
	if (LDS_LAYOUT == COC_TILE_LAYOUT_FGD_SEP_BGD)
	{
		Shared0[i] = float4(Raw[0].rg, Raw[1].rg);
		Shared1[i] = Raw[1].b;
	}
	else // if (LDS_LAYOUT == COC_TILE_LAYOUT_FGD_BGD)
	{
		Shared0[i] = Raw[0];
	}
}


#else

groupshared uint SharedForegroundMinCocRadius;
groupshared uint SharedForegroundMaxCocRadius;
groupshared uint SharedBackgroundMinCocRadius;
groupshared uint SharedBackgroundMinIntersectableCocRadius;
groupshared uint SharedBackgroundMaxCocRadius;

#endif


//------------------------------------------------------- ENTRY POINT.

[numthreads(DEFAULT_GROUP_BORDER_SIZE, DEFAULT_GROUP_BORDER_SIZE, 1)]
void CocFlattenMainCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint GroupIndex : SV_GroupIndex)
{
	ResolvedView = ResolveView();

	FCocTileSample TileSample;

	#if DIM_DO_COC_GATHER4
	{
		float2 BufferUV;
		{
			float2 PixelPosition = (DispatchThreadId * Stride) + 0.5;
			BufferUV = GatherInputViewRectUV.xy + (PixelPosition * GatherInputExtentAndInvExtent.zw);
			BufferUV = clamp(BufferUV, GatherInputViewRectUV.xy, GatherInputViewRectUV.zw);
		}
		
		#if CONFIG_DOF_ALPHA
			// The coc radius is within separate buffer's red channel.
			float4 CocRadii = GatherInput_SeparateCoc.GatherRed(GlobalPointClampedSampler, BufferUV, 0);
		#else
			float4 CocRadii = GatherInput_SceneColor.GatherAlpha(GlobalPointClampedSampler, BufferUV, 0);
		#endif

		InitCocTileSampleFromCocRadius(CocRadii.x, TileSample);

		UNROLL
		for (uint i = 1; i < 4; i++)
		{
			FCocTileSample SecondTileSample;
			InitCocTileSampleFromCocRadius(CocRadii[i], SecondTileSample);
			ReduceOperator(TileSample, SecondTileSample);
		}
	}
	#else // !DIM_DO_COC_GATHER4
	{
		uint2 BufferPixelPosition;
		{
			float2 PixelPosition = DispatchThreadId + 0.5;
			float2 BufferUV = GatherInputViewRectUV.xy + (PixelPosition * GatherInputExtentAndInvExtent.zw);
			BufferUV = clamp(BufferUV, GatherInputViewRectUV.xy, GatherInputViewRectUV.zw);
			BufferPixelPosition = BufferUV * GatherInputExtentAndInvExtent.xy;
		}

		#if CONFIG_DOF_ALPHA
			// The coc radius is within separate buffer's red channel.
			float CocRadius = GatherInput_SeparateCoc[BufferPixelPosition].r;
		#else
			float CocRadius = GatherInput_SceneColor[BufferPixelPosition].a;
		#endif

		InitCocTileSampleFromCocRadius(CocRadius, TileSample);
	}
	#endif

	#if CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_REDUCE_LDS
	{
		// Setup local sample.
		FCocTileSample Local = TileSample;

		// LDS setup.
		StoreToLds(GroupIndex, Local);
		GroupMemoryBarrierWithGroupSync();
	
		// Do reduction.
		// TODO: atomics.
		UNROLL
		for (uint j = 0; j < 5; j++)
		{
			const uint BankSize = 1 << (5 - j);

			BRANCH
			if (GroupIndex < BankSize)
			{
				FCocTileSample Other = LoadFromLds(GroupIndex + BankSize);
				ReduceOperator(Local, Other);
				StoreToLds(GroupIndex, Local);
			}
		
			GroupMemoryBarrierWithGroupSync();
		}

		if (GroupIndex == 0)
		{	
			// Final reduction.
			ReduceOperator(Local, LoadFromLds(1));
			
			Local.BgdMinConservativeCloserGoemetryDistance = Local.BgdMinCoc;

			// Encode.
			float4 RawOutput[COC_TILE_MAX_BUFFER_COUNT];
			EncodeCocTileSample(Local, LDS_LAYOUT, RawOutput);
	
			// Output.
			TileOutput_Foreground[GroupId] = RawOutput[0];
			if (static_condition(GetBufferCountForCocTileLayour(LDS_LAYOUT) > 1))
			{
				TileOutput_Background[GroupId] = RawOutput[1];
			}
		}
	}
	#elif CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_ATOMIC_LDS || CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_WAVE_MINMAX
	{
		// Init LDS if the group size and wave size doesn't match.
		if (WaveGetLaneCount() != DEFAULT_GROUP_BORDER_SIZE * DEFAULT_GROUP_BORDER_SIZE)
		{
			BRANCH
			if (GroupIndex == 0)
			{
				SharedForegroundMinCocRadius = 0;
				SharedForegroundMaxCocRadius = ~0;
				SharedBackgroundMinCocRadius = ~0;
				SharedBackgroundMinIntersectableCocRadius = ~0;
				SharedBackgroundMaxCocRadius = 0;
			}
		
			GroupMemoryBarrierWithGroupSync();
		}

		// Perform wave flattening first.
		#if CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_WAVE_MINMAX
			TileSample.FgdMinCoc = WaveActiveMin(TileSample.FgdMinCoc);
			TileSample.FgdMaxCoc = WaveActiveMax(TileSample.FgdMaxCoc);
			TileSample.BgdMinCoc = WaveActiveMin(TileSample.BgdMinCoc);
			TileSample.BgdMaxCoc = WaveActiveMax(TileSample.BgdMaxCoc);
			TileSample.BgdMinIntersectableCoc = WaveActiveMin(TileSample.BgdMinIntersectableCoc);

			if (WaveGetLaneCount() == DEFAULT_GROUP_BORDER_SIZE * DEFAULT_GROUP_BORDER_SIZE)
			{
				// NOP: because no need to gather the result of all waves of the group, because there is
				// only one wave per group.
			}
			else if (WaveIsFirstLane())
		#endif
		
		// Atomic LDS reduce by reinterpret casting float as uint.
		{
			const uint RangeId = 0;

			uint Unused;
			
			// Foreground CoCRadius <= 0.0.
			InterlockedMax(SharedForegroundMinCocRadius, asuint(TileSample.FgdMinCoc), Unused);
			InterlockedMin(SharedForegroundMaxCocRadius, asuint(TileSample.FgdMaxCoc), Unused);

			// Background CoCRadius >= 0.0.
			InterlockedMin(SharedBackgroundMinCocRadius, asuint(TileSample.BgdMinCoc), Unused);
			InterlockedMax(SharedBackgroundMaxCocRadius, asuint(TileSample.BgdMaxCoc), Unused);
			InterlockedMin(SharedBackgroundMinIntersectableCocRadius, asuint(TileSample.BgdMinIntersectableCoc), Unused);
			
			GroupMemoryBarrierWithGroupSync();
		}

		// Output.
		BRANCH
		if (GroupIndex == 0)
		{	
			// Finialize tile sample to output.
			FCocTileSample OutputTileSample;
			#if CONFIG_FLATTEN_METHOD == FLATTEN_METHOD_WAVE_MINMAX
				if (WaveGetLaneCount() == DEFAULT_GROUP_BORDER_SIZE * DEFAULT_GROUP_BORDER_SIZE)
				{
					OutputTileSample = TileSample;
				}
				else
			#endif
			{
				OutputTileSample.FgdMinCoc = asfloat(SharedForegroundMinCocRadius);
				OutputTileSample.FgdMaxCoc = asfloat(SharedForegroundMaxCocRadius);
				OutputTileSample.BgdMinCoc = asfloat(SharedBackgroundMinCocRadius);
				OutputTileSample.BgdMaxCoc = asfloat(SharedBackgroundMaxCocRadius);
				OutputTileSample.BgdMinIntersectableCoc = asfloat(SharedBackgroundMinIntersectableCocRadius);
			}

			OutputTileSample.BgdMinConservativeCloserGoemetryDistance = OutputTileSample.BgdMinCoc;

			// Encode.
			float4 RawOutput[COC_TILE_MAX_BUFFER_COUNT];
			EncodeCocTileSample(OutputTileSample, LDS_LAYOUT, RawOutput);
	
			// Output.
			TileOutput_Foreground[GroupId] = RawOutput[0];
			if (static_condition(GetBufferCountForCocTileLayour(LDS_LAYOUT) > 1))
			{
				TileOutput_Background[GroupId] = RawOutput[1];
			}
		}
	}
	#else
		#error Unknown flatten method.
	#endif
}
