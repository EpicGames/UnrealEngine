// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFCommon.usf: Common shader code for diaphragm DOF.
=============================================================================*/

#pragma once

#include "../Common.ush"
#include "../ScreenPass.ush"

// Piggy back on existing Circle DOF code.
#include "../CircleDOFCommon.ush"


//------------------------------------------------------- RECOMPILE HASH

#pragma message("UESHADERMETADATA_VERSION EADEBBCC-57EB-41E0-9606-DC289F55A057")


//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)

// Generate error when comparing int and uint.
#pragma warning(error: 3203)

// Ignores race condition detection when writing to LDS.
#pragma warning(disable: 3584)

// Equality comparison with extraneous parentheses
#pragma dxc diagnostic ignored "-Wparentheses-equality"


//------------------------------------------------------- Readability helpers.

// Code readability improvement to make explicit compile time branches like so:
//
// if (static_condition(MyBool))
// {
//   //[...]
// }
#define static_condition(test) (test)


//------------------------------------------------------- DIMENSIONS VALUES.

// Matches C++'s EDiaphragmDOFLayerProcessing
#define LAYER_PROCESSING_FOREGROUND_ONLY 0
#define LAYER_PROCESSING_FOREGROUND_HOLE_FILLING 1
#define LAYER_PROCESSING_BACKGROUND_ONLY 2
#define LAYER_PROCESSING_COMBINED 3
#define LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS 4


// Matches C++'s EDiaphragmDOFBokehSimulation
#define BOKEH_SIMULATION_DISABLED 0
#define BOKEH_SIMULATION_SIMMETRIC 1
#define BOKEH_SIMULATION_GENERAL 2


//------------------------------------------------------- ENUM VALUES

/** Different hole filling methods. */
	// Hole filling technic that directly changes the foreground opacity. Simplest but hugliest.
	#define HOLE_FILLING_METHOD_OPACITY_AMEND 0
	
	// Use's foreground gathering pass.
	#define HOLE_FILLING_METHOD_SEPARATE_GATHER 1
	
	// TODO IDEA: bilateral sample foreground according to opacity.
	#define HOLE_FILLING_METHOD_BILATERAL 2
	
/** Layout of the input of the gathering passes. */
	// Scene color and CocRadius stored within RGBA buffer 0.
	#define GATHER_INPUT_LAYOUT_RGB_COC 0
	
	// Scene color and alpha stored within RGBA buffer 0, and Coc in buffer 1.
	#define GATHER_INPUT_LAYOUT_RGB_ALPHA_COC 1
	
	// Scene color within RGB_11_11_10 buffer 0, and Coc in buffer 1.
	#define GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC 2


//------------------------------------------------------- COMPILE TIME CONSTANTS.

// A very large Coc radius that is assumed to be unreachable, but still encodable in 16bits float.
#define EXTREMELY_LARGE_COC_RADIUS 16384


// Default border size for group.
#define DEFAULT_GROUP_BORDER_SIZE 8

// Size of the COC in pixel from full screen.
#define COC_TILE_SIZE 8


// The resolution divisor of the current pass
#define PASS_RES_DIVISOR 2

// The resolution devisor that passed as an input of the Coc flatten.
#define COC_TILE_RES_DIVISOR 2

// Number of rings on the main kernel.
#define MAIN_KERNEL_RING_COUNT 5

// Maximum abs(coc radius) the full res recombine pass is going to do.
#define MAX_RECOMBINE_ABS_COC_RADIUS 3.0

/** Width and height of the bokeh LUT. */
#define BOKEH_LUT_SIZE 32

/** Whether alpha channel should be processed or not. */
#ifdef DIM_ALPHA_CHANNEL
	#define CONFIG_DOF_ALPHA (DIM_ALPHA_CHANNEL)
#else
	#define CONFIG_DOF_ALPHA 0
#endif

/** Amount of error tolerated to quick fast gathering in. */
#define FAST_GATHERING_COC_ERROR 0.05

/** Whether to output a debug texture for passes. */
#define DEBUG_OUTPUT 0


//------------------------------------------------------- COMPILE TIME CONSTANTS.

// Coc radiuses are in half res.
static const float kCocRadiusToFullResFactor = 0.5;


static const int2 kSquare2x2[4] = {
	int2(0, 0),
	int2(1, 0),
	int2(0, 1),
	int2(1, 1),
};

static const int2 kOffsetsCross3x3[4] = {
	int2(-1, -1),
	int2( 1, -1),
	int2(-1,  1),
	int2( 1,  1),
};

static const int2 kOffsetsSquare3x3[9] = {
	int2(-1, +1),
	int2(-1,  0),
	int2(-1, -1),
	int2( 0, +1),
	int2( 0,  0),
	int2( 0, -1),
	int2(+1, +1),
	int2(+1,  0),
	int2(+1, -1),
};


//------------------------------------------------------- PARAMETERS

// See SetCocModelParameters, CocRadiusBasis
float CocInfinityRadius;
float CocInFocusRadius;
uint bCocEnableDynamicRadiusOffset;
float CocMinRadius;
float CocMaxRadius;
float CocSqueeze;
float CocInvSqueeze;

float DepthBlurRadius;
float DepthBlurExponent;

Texture2D		DynamicRadiusOffsetLUT;
SamplerState	DynamicRadiusOffsetLUTSampler;

//------------------------------------------------------- DEBUG OUTPUT

#if DEBUG_OUTPUT
RWTexture2D<float4> DebugOutput;
#endif


//------------------------------------------------------- COMMON API.

#if WITH_DYNAMIC_COC_OFFSET
float GetCocOffset(float CocRadius)
{
	float DynamicOffset = 0.0;
	BRANCH if (bCocEnableDynamicRadiusOffset)
	{
		// The 1D LUT is indexed by ratio of object CoC to in-focus CoC, starting at ratio of 1 (since we only care about objects at or beyond the in-focus plane)
		// and going to a maximum ratio calculated by the number of pixels in the LUT and the starting object and wall CoCs. By convention, the LUT has been
		// computed with starting CoC of 32
		float MaxRatio = (256 + 32) / 32;
		float Ratio = CocInFocusRadius > 0 ? CocRadius / CocInFocusRadius : MaxRatio;
		float2 TexCoords = float2((Ratio - 1) / (MaxRatio - 1), 0);
		float Offset = DynamicRadiusOffsetLUT.SampleLevel(DynamicRadiusOffsetLUTSampler, TexCoords, 0).r * CocInFocusRadius;
		DynamicOffset = -min(Offset, CocRadius);
	}

	return DynamicOffset;
}
#endif

// Returns the coc radius
// The unit for the output, i.e. 'encoded' pixels or physical pixels, depends on what CocRadiusBasis was used.
float SceneDepthToCocRadius(float SceneDepth)
{
	const float Focus = View.DepthOfFieldFocalDistance; // TODO: rename this to DepthOfFieldFocusDistance.
	
	// Radius from the circle of confusion.
	float CocRadius = ((SceneDepth - Focus) / SceneDepth) * CocInfinityRadius;

#if WITH_DYNAMIC_COC_OFFSET
	CocRadius = CocRadius + GetCocOffset(CocRadius);
#endif

	// Depth blur's radius.
	float DepthBlurAbsRadius = (1.0 - exp2(-SceneDepth * DepthBlurExponent)) * DepthBlurRadius;

	float ReturnCoc = max(abs(CocRadius), DepthBlurAbsRadius) * sign(CocRadius);

	return clamp(ReturnCoc, CocMinRadius, CocMaxRadius);
}

// Return whether this is front element from CocRadius.
bool IsForeground(float CocRadius)
{
	return CocRadius < 0.0f;
}

// Faster but less accurate luma computation with a scaling by 4.
float Luma4(float3 Color)
{
	return (Color.g * 2.0) + (Color.r + Color.b);
}

// Returns 1/x if x > 0, d otherwise.
float SafeRcp(float x, float d = 0.0)
{
	return x > 0.0 ? rcp(x) : d;
}

// Converts a distance in full res pixel to coc distance.
#define FullResPixelDistanceToCocDistance(x) (0.5 * (x))

// Converts a distance in full res pixel to coc distance.
#define CocDistanceToFullResPixelDistance(x) (2.0 * (x))

// Utility to use a saturating affine transformation.
float SaturateWithAffineTransformation(float x, float2 AffineTransformation)
{
	// Hint: Just one MAD with saturate post modifier on GCN.
	return saturate(x * AffineTransformation.x + AffineTransformation.y);
}

// Affine transformtations that always return 0 or 1.
#define kContantlyPassingAffineTransformation float2(0, 1)
#define kContantlyBlockingAffineTransformation float2(0, 0)

#if 0 // TODO: COD's noise (slide 79)
#endif
