// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFDownsample.usf: Diaphragm DOF's bilateral downsampling shader. 
=============================================================================*/

#include "DOFDownsample.ush"


//------------------------------------------------------- PARAMETERS

uint4 ViewportRect;
float OutputCocRadiusMultiplier;

float4 GatherInputExtentAndInvExtent;
float4 GatherInputViewRectUV;
Texture2D GatherInput_SceneColor;
Texture2D GatherInput_SeparateCoc;


//------------------------------------------------------- OUTPUTS

RWTexture2D<float4>	OutDownsampledGatherInput_SceneColor;

#if CONFIG_DOF_ALPHA
RWTexture2D<float>	OutDownsampledGatherInput_SeparateCoc;
#endif


//------------------------------------------------------- ENTRY POINT

[numthreads(DEFAULT_GROUP_BORDER_SIZE, DEFAULT_GROUP_BORDER_SIZE, 1)]
void DownsampleCS(uint2 DispatchThreadId : SV_DispatchThreadID) 
{
	ResolvedView = ResolveView();

	// Compute scene buffer uv, but carefully to handle view size % 2 == 1,
	// because this pass is running at half resolution.
	float2 PixelPosition = 2.0 * DispatchThreadId;
	float2 BufferUV = GatherInputViewRectUV.xy + ((PixelPosition + 0.5) * GatherInputExtentAndInvExtent.zw);
	
	// Sample scene color.
	float4 SceneColors[4];
	float CocRadii[4];

	for (uint i = 0; i < 4; i++)
	{
		float2 SampleUV = BufferUV + GatherInputExtentAndInvExtent.zw * (0.5 * float2(kOffsetsCross3x3[i]));

		SampleUV = clamp(SampleUV, GatherInputViewRectUV.xy, GatherInputViewRectUV.zw);

		float4 RawSample0 = GatherInput_SceneColor.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);

		#if CONFIG_DOF_ALPHA
		{
			float4 RawSample1 = GatherInput_SeparateCoc.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);

			SceneColors[i] = RawSample0;
			CocRadii[i] = RawSample1.r;
		}
		#else
		{
			SceneColors[i] = float4(RawSample0.rgb, 0);
			CocRadii[i] = RawSample0.a;
		}
		#endif
	}
	
	FCocDownsampleParams DownsampleParams;
	DownsampleParams.CocRadiusMultiplier = 1.0;
	DownsampleParams.FrameExposureScale = 1.0;
	DownsampleParams.bDoColorBasedWeighting = false;

	// Downsample.
	float4 OutColor;
	float OutCocRadius;
	DownsampleSceneColorWithCoc(DownsampleParams, SceneColors, CocRadii, OutColor, OutCocRadius);
	
	if (all(DispatchThreadId < ViewportRect.zw))
	{
		#if CONFIG_DOF_ALPHA
		{
			OutDownsampledGatherInput_SceneColor[DispatchThreadId] = OutColor;
			OutDownsampledGatherInput_SeparateCoc[DispatchThreadId] = OutCocRadius * OutputCocRadiusMultiplier;
		}
		#else
		{
			OutDownsampledGatherInput_SceneColor[DispatchThreadId] = float4(OutColor.rgb, OutCocRadius * OutputCocRadiusMultiplier);
		}
		#endif
	}
}
