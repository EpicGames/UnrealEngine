// Copyright Epic Games, Inc. All Rights Reserved.

/*============================================================================
	DiaphragmDOF/DOFDownsample.ush: Diaphragm DOF's downsampling code/
=============================================================================*/

#include "DOFCommon.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"


//------------------------------------------------------- COMPILE TIME CONFIG

#if defined(CONFIG_SETUP)
#define DOWNSAMPLE_COC_OPERATOR 4
#else
// Defines the COC operator to choose when downsampling.
// Same as circle DOF.
#define DOWNSAMPLE_COC_OPERATOR 2
#endif


//------------------------------------------------------- COMMON FUNCTIONS.

/** Operator to use to downsample 4 sample to 1. */
// TODO: This as been fetched from Circle DOF, so redo this experiments, and try new ones.
float DownsampleCoc(float CocRadii[4])
{
	// Doing a max depth reduction (erode the foreground). Less correct, but less artifacts.
	// Perhaps need to re-open this in the future.
	float mi = min(min(CocRadii[0], CocRadii[1]), min(CocRadii[2], CocRadii[3]));
	float ma = max(max(CocRadii[0], CocRadii[1]), max(CocRadii[2], CocRadii[3]));
	float ami = min(min(abs(CocRadii[0]), abs(CocRadii[1])), min(abs(CocRadii[2]), abs(CocRadii[3])));
	float ama = max(max(abs(CocRadii[0]), abs(CocRadii[1])), max(abs(CocRadii[2]), abs(CocRadii[3])));

	#if DOWNSAMPLE_COC_OPERATOR == 0
		// Stuff max radius in alpha.
		// bad erosion on TemporalDitherAA
		float OutCocRadius = ma;

	#elif DOWNSAMPLE_COC_OPERATOR == 1
		// acceptable TemporalDitherAA
		// requires DefaultWeight > 1
		float OutCocRadius = (mi + ma) / 2;

	#elif DOWNSAMPLE_COC_OPERATOR == 2
		// This in theory is better but causes bleeding artifacts with temporal AA..
		// This is important otherwise near thin objects disappear (leaves clamping artifacts in recombined pass).
		// bad on TemporalDitherAA, flat opacity where it should transition
		float OutCocRadius = CocRadii[0];
		if(abs(OutCocRadius) > CocRadii[1]) OutCocRadius = CocRadii[1];
		if(abs(OutCocRadius) > CocRadii[2]) OutCocRadius = CocRadii[2];
		if(abs(OutCocRadius) > CocRadii[3]) OutCocRadius = CocRadii[3];

	#elif DOWNSAMPLE_COC_OPERATOR == 3
		// this should be better than the method before
		// bad on TemporalDitherAA
		float OutCocRadius = CocRadii[0];
		if(abs(OutCocRadius) > abs(CocRadii[1])) OutCocRadius = CocRadii[1];
		if(abs(OutCocRadius) > abs(CocRadii[2])) OutCocRadius = CocRadii[2];
		if(abs(OutCocRadius) > abs(CocRadii[3])) OutCocRadius = CocRadii[3];

	#elif DOWNSAMPLE_COC_OPERATOR == 4
		// Stuff max radius in alpha.
		float OutCocRadius = mi;

	#elif DOWNSAMPLE_COC_OPERATOR == 5
		// artifacts that look like negative colors (tb070) (with and without the 2nd line)
		// bad erosion on TemporalDitherAA
		float OutCocRadius = (ami + ama) / 2;

	#elif DOWNSAMPLE_COC_OPERATOR == 6
		// like #3 but with inverted comparison, ok?
		// bad erosion on TemporalDitherAA
		float OutCocRadius = CocRadii[0];
		if(abs(OutCocRadius) < abs(CocRadii[1])) OutCocRadius = CocRadii[1];
		if(abs(OutCocRadius) < abs(CocRadii[2])) OutCocRadius = CocRadii[2];
		if(abs(OutCocRadius) < abs(CocRadii[3])) OutCocRadius = CocRadii[3];

	#elif DOWNSAMPLE_COC_OPERATOR == 7
		// requires DefaultWeight > 1
		float A = CocRadii[0];
		if(abs(A) < abs(CocRadii[1])) A = CocRadii[1];
		if(abs(A) < abs(CocRadii[2])) A = CocRadii[2];
		if(abs(A) < abs(CocRadii[3])) A = CocRadii[3];
		float B = CocRadii[0];
		if(abs(B) > abs(CocRadii[1])) B = CocRadii[1];
		if(abs(B) > abs(CocRadii[2])) B = CocRadii[2];
		if(abs(B) > abs(CocRadii[3])) B = CocRadii[3];
		float OutCocRadius = (A + B) / 2;

	#elif DOWNSAMPLE_COC_OPERATOR == 8
		// broken near dof
		float OutCocRadius = dot(0.25f, max(0, CocQuad));

	#elif DOWNSAMPLE_COC_OPERATOR == 9
		// mix between 2 and 8, seems to be best in most cases
		// requires DefaultWeight > 1
		float OutCocRadius = CocRadii[0];
		if(abs(OutCocRadius) > CocRadii[1]) OutCocRadius = CocRadii[1];
		if(abs(OutCocRadius) > CocRadii[2]) OutCocRadius = CocRadii[2];
		if(abs(OutCocRadius) > CocRadii[3]) OutCocRadius = CocRadii[3];
		if(OutCocRadius > 0)	OutCocRadius = dot(0.25f, max(0, CocQuad));

	#else
		#error unknown DOWNSAMPLE_COC_OPERATOR.

	#endif

	return OutCocRadius;
}


// Optimized HDR weighting function.
float HdrWeight4(float3 Color, float Exposure) 
{
	Exposure *= View.PreExposure;

	return rcp(Luma4(Color) * Exposure + 4.0);
}


struct FCocDownsampleParams
{
	// Multiplier to apply on the bilateral weights.
	float CocRadiusMultiplier;

	// The exposure scale of the frame.
	float FrameExposureScale;

	// Whether should color based weightings to reduce highlight contamination.
	bool bDoColorBasedWeighting;
};


/** Compute the bilateral weight of sample to downsample. */
float ComputeDownsamplingBilateralWeight(
	FCocDownsampleParams DownsampleParams,
	float OutCocRadius,
	float SampleCocRadius,
	float3 SampleColor = 0)
{
	// Remove samples which are outside the size.
	// TODO: Tune the ScaleFactor. Looks like to large.
	float ScaleFactor = 64.0 * DownsampleParams.CocRadiusMultiplier;

	// not doing abs(OutCocRadius - SampleCocRadius) because: it is fine to leak background on foreground arround geometric
	// edges because going to be close to the hole filling as well. This allow dither opacity material such as dithered human hair
	// to keep consistent opacity, thickness, and temporal stability as the Coc changes between slight out focus, foreground
	// and background.
	#if 1
		float BilateralWeight = saturate(1.0 - (OutCocRadius - SampleCocRadius) * ScaleFactor);
	#else
		// GCN Hint: one subtract with abs() post modifier + one MAD with saturate post modifier.
		float BilateralWeight = saturate(1.0 - abs(OutCocRadius - SampleCocRadius) * ScaleFactor);
	#endif
	
	float ColorWeight = 1;
	if (DownsampleParams.bDoColorBasedWeighting)
	{
		ColorWeight = HdrWeight4(SampleColor, DownsampleParams.FrameExposureScale);
	}

	return BilateralWeight * ColorWeight;
}


/** Operator to use to downsample 4 scene sample to 1. */
void DownsampleSceneColorWithCoc(
	FCocDownsampleParams DownsampleParams,
	float4 Color[4], float CocRadii[4],
	out float4 OutColor, out float OutCocRadius)
{
	// Choose the best the coc to use.
	OutCocRadius = DownsampleCoc(CocRadii);

	float4 BilateralWeights = float4(
		ComputeDownsamplingBilateralWeight(DownsampleParams, OutCocRadius, CocRadii[0], Color[0].rgb),
		ComputeDownsamplingBilateralWeight(DownsampleParams, OutCocRadius, CocRadii[1], Color[1].rgb),
		ComputeDownsamplingBilateralWeight(DownsampleParams, OutCocRadius, CocRadii[2], Color[2].rgb),
		ComputeDownsamplingBilateralWeight(DownsampleParams, OutCocRadius, CocRadii[3], Color[3].rgb));

	float WeightSum = dot(float4(1, 1, 1, 1), BilateralWeights);

	// Normalize weights.
	float WeightNormalizationFactor = rcp(WeightSum);

	// Do the multiply of WeightNormalizationFactor, because save one mad when alpha channel is disabled.
	OutColor = WeightNormalizationFactor * (
		Color[0] * BilateralWeights.x +
		Color[1] * BilateralWeights.y +
		Color[2] * BilateralWeights.z +
		Color[3] * BilateralWeights.w);
}


#if PLATFORM_SUPPORTS_WAVE_BROADCAST

// Returns the downsampling Coc from the Coc radius on 4 different lane:
//	(0; 0)
//	(ReductionScale; 0)
//	(0; ReductionScale)
//	(ReductionScale; ReductionScale)
float DownsampleCoc_Wave8x8(float CocRadius, const uint ReductionScale)
{
	float OutCocRadius = CocRadius;
	
	#if DOWNSAMPLE_COC_OPERATOR == 2
		// Horizontal.
		float OtherCocRadius = WaveBroadcast(
			InitWaveBroadcastLaneGroup(
				/* LaneGroupSize = */ 2 * ReductionScale,
				/* InnerLaneGroupSize = */ 1 * ReductionScale,
				/* InnerLaneGroupId = */ 1),
			OutCocRadius);
		if(abs(OutCocRadius) > OtherCocRadius) OutCocRadius = OtherCocRadius;

		// Vertical.
		OtherCocRadius = WaveBroadcast(
			InitWaveBroadcastLaneGroup(
				/* LaneGroupSize = */ 16 * ReductionScale,
				/* InnerLaneGroupSize = */ 8 * ReductionScale,
				/* InnerLaneGroupId = */ 1),
			OutCocRadius);
		if(abs(OutCocRadius) > OtherCocRadius) OutCocRadius = OtherCocRadius;

		// Broad cast back to ensure identical OutCocRadius.
		{
			OutCocRadius = WaveBroadcast(
				InitWaveBroadcastLaneGroup(
					/* LaneGroupSize = */ 2 * ReductionScale,
					/* InnerLaneGroupSize = */ 1 * ReductionScale,
					/* InnerLaneGroupId = */ 0),
				OutCocRadius);
			
			OutCocRadius = WaveBroadcast(
				InitWaveBroadcastLaneGroup(
					/* LaneGroupSize = */ 16 * ReductionScale,
					/* InnerLaneGroupSize = */ 8 * ReductionScale,
					/* InnerLaneGroupId = */ 0),
				OutCocRadius);
		}
	#elif DOWNSAMPLE_COC_OPERATOR == 4
		// Horizontal.
		float OtherCocRadius = WaveBroadcast(
			InitWaveSwapWithinLaneGroup(/* LaneGroupSize = */ 2 * ReductionScale),
			OutCocRadius);
		OutCocRadius = min(OutCocRadius, OtherCocRadius);
		
		// Vertical.
		OtherCocRadius = WaveBroadcast(
			InitWaveSwapWithinLaneGroup(/* LaneGroupSize = */ 16 * ReductionScale),
			OutCocRadius);
		OutCocRadius = min(OutCocRadius, OtherCocRadius);
	#else
		#error Unimplemented.
	#endif
	
	return OutCocRadius;
}

// Sums values on 4 different lane when dispatched as 8x8 tile:
//	(0; 0)
//	(ReductionScale; 0)
//	(0; ReductionScale)
//	(ReductionScale; ReductionScale)
float Sum2x2WithinWave8x8(float x, const uint ReductionScale)
{
	float y = x + WaveBroadcast(
		InitWaveBroadcastLaneGroup(
			/* LaneGroupSize = */ 2 * ReductionScale,
			/* InnerLaneGroupSize = */ ReductionScale,
			/* InnerLaneGroupId = */ 1),
		x);

	float z = y + WaveBroadcast(
		InitWaveBroadcastLaneGroup(
			/* LaneGroupSize = */ 16 * ReductionScale,
			/* InnerLaneGroupSize = */ 8 * ReductionScale,
			/* InnerLaneGroupId = */ 1),
		y);
	return z;
}

#endif // PLATFORM_SUPPORTS_WAVE_BROADCAST
