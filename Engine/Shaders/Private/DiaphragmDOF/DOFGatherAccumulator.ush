// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFGatherAccumulator.usf: gather accumulator for diaphragm DOF. 
=============================================================================*/

#pragma once

#include "DOFGatherCommon.ush"


//------------------------------------------------------- GATHER ACCUMULATOR

/** Accumulator of all gathered samples. */
struct FGatherAccumulator
{
	// Gather parameters.
	FGatherInputParameters GatherParameters;

	// Gathering kernel for larger.
	FLargerConvolution LargerConvolution;
	
	// Layer to process.
	uint LayerProcessing;

	#if GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_SIMPLE
		// One single layer.
		float4 Color;
		float ColorWeight;

		float Opacity;
		float OpacityWeight;

		// Distance to the closest foreground sample.
		float DistanceToClosestForeground;

	#elif GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_JIMENEZ2014
		// Two layers, 0 is the closest to camera.
	
		// Hole filling?
		bool bIsHoleFillingForForeground;

		// Coc radius of the center of the kernel.
		float CenterCocRadius;

		// Scene color per layer.
		float4 Layer[2];

		// Weight of Layer.
		float LayerWeight[2];

		// Number of samples accumulated in each layers.
		float LayerSampleCount[2];

		float IntersectionSum;
		float ForegroundSum;
		
	#elif GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING
		// Accumulated results for background reconstruction.
		// float4 BackgroundReconstruction; // TODO.

		// Accumulated results for previous bucket.
		float4 PreviousColor;
		float PreviousCocRadius;
		float PreviousCocRadiusSquare;
		float PreviousWeight;

		// Accumulating results for current bucket.
		float4 CurrentColor;
		float CurrentCocRadius;
		float CurrentCocRadiusSquare;
		float CurrentWeight;
		
		float CurrentTranslucency;

		// The bordering radius between current and previous buckets.
		float BorderingRadius;

		bool bIsFirstRing;
		
	#else
		#error Unknown gather accumulator.

	#endif

	#if CONFIG_LAYER_GATHERING 
		float FarBorderingRadius;
		float CloseBorderingRadius;
	#endif
};

/** Create an accumulator for gathering. */
FGatherAccumulator CreateGatherAccumulator(
	in FGatherInputParameters GatherParameters,
	in FLargerConvolution LargerConvolution)
{
	FGatherAccumulator Accumulator;
	
	Accumulator.GatherParameters = GatherParameters;

	// Always store larger convolution in accumulator, in case want to compose larger convolution
	// in ResolveForegroundAndBackgroundOutputs().
	Accumulator.LargerConvolution = LargerConvolution;

	Accumulator.LayerProcessing = DIM_LAYER_PROCESSING;

	#if GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_JIMENEZ2014
	{
		Accumulator.IntersectionSum = 0;
		Accumulator.ForegroundSum = 0;
		Accumulator.bIsHoleFillingForForeground = false;
		Accumulator.CenterCocRadius = 0.0;
		for (uint k = 0; k < 2; k++)
		{
			Accumulator.Layer[k] = 0;
			Accumulator.LayerWeight[k] = 0;
			Accumulator.LayerSampleCount[k] = 0;
		}
	}
	#elif GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING
	{
		Accumulator.PreviousColor = 0;
		Accumulator.PreviousCocRadius = 0;
		Accumulator.PreviousCocRadiusSquare = 0;
		Accumulator.PreviousWeight = 0;

		Accumulator.CurrentColor = 0;
		Accumulator.CurrentCocRadius = 0;
		Accumulator.CurrentCocRadiusSquare = 0;
		Accumulator.CurrentWeight = 0;

		Accumulator.CurrentTranslucency = 0;
	}
	#elif GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_SIMPLE
	{
		Accumulator.Color = 0;
		Accumulator.ColorWeight = 0;
		Accumulator.Opacity = 0;
		Accumulator.OpacityWeight = 0;
		Accumulator.DistanceToClosestForeground = EXTREMELY_LARGE_COC_RADIUS;
	}
	#endif
	
	#if CONFIG_LAYER_GATHERING
	{
		Accumulator.FarBorderingRadius = 0;
		Accumulator.CloseBorderingRadius = 0;
	}
	#endif

	return Accumulator;
}


//------------------------------------------------------- Simple standalone bin accumulator.
#if GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_SIMPLE

struct FGatherSampleDerivedParameters
{
	float Weight;
	float IsConsidered;
	float Opacity;
};

// Returns the opacity to use to transition to background.
float ComputeBackgroundSampleOpacity(in const FGatherInputParameters GatherParameters, float CocRadius)
{
	//return CocRadius < GatherParameters.MaxRecombineAbsCocRadius ? 1 : 0;
	return saturate(GatherParameters.MaxRecombineAbsCocRadius - CocRadius);
}

FGatherSampleDerivedParameters ComputeSampleDerivates(in FGatherAccumulator Accumulator, in FGatherSample A)
{
	FGatherSampleDerivedParameters DerivedA;
	DerivedA.Weight = ComputeSampleWeight(A.CocRadius);
	
	//if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_ONLY)
	//{
	//	DerivedA.Opacity = ComputeForegroundSampleOpacity(A.CocRadius);
	//	DerivedA.IsConsidered = IsConsideredForegroundSample(A.CocRadius);
	//}
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		// Don't care about weight for hole filling
		DerivedA.Weight = 1;
	
		DerivedA.Opacity = 1;
		//DerivedA.Opacity = ComputeForegroundSampleOpacity(A.CocRadius);
		DerivedA.IsConsidered = IsConsideredCocRadius(Accumulator.GatherParameters, A.CocRadius);
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		DerivedA.Opacity = ComputeBackgroundSampleOpacity(Accumulator.GatherParameters, A.CocRadius);
		DerivedA.IsConsidered = saturate(Accumulator.GatherParameters.MaxRecombineAbsCocRadius - abs(A.CocRadius));
	}
	//else if (Accumulator.LayerProcessing == LAYER_PROCESSING_BACKGROUND_ONLY)
	//{
	//	DerivedA.Opacity = ComputeBackgroundSampleOpacity(A.CocRadius);
	//	DerivedA.IsConsidered = IsConsideredBackgroundSample(A.CocRadius);
	//}

	return DerivedA;
}

void HoleFillCloserSample(
	in FGatherAccumulator Accumulator,
	inout FGatherSample A, inout FGatherSampleDerivedParameters DerivedA,
	in FGatherSample Closer, in FGatherSampleDerivedParameters DerivedCloser)
{
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_ONLY)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;

		#if 1 // Used with LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
				
		#elif 0 // looks nice over slight out of focus, but looks bad over large background out of focus.
			Opacity[1] = Opacity[0] * ComputeBackgroundSampleOpacity(Accumulator.GatherParameters, S[1].CocRadius);
			IsConsidered[1] = Opacity[1] * IsConsidered[0];

		#else
			DerivedA.IsConsidered = DerivedCloser.IsConsidered;
			DerivedA.Opacity = DerivedCloser.Opacity;
		#endif
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;
		
		DerivedA.Opacity = DerivedCloser.Opacity;
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;
	}
}

/** Accumulates sample. */
void AccumulateSample(
	inout FGatherAccumulator Accumulator,
	in FGatherSample A,
	in FGatherSampleDerivedParameters DerivedA)
{
	// TODO: soften that.
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING &&
		A.CocRadius < -(Accumulator.GatherParameters.MaxRecombineAbsCocRadius - 1))
	{
		Accumulator.DistanceToClosestForeground = min(Accumulator.DistanceToClosestForeground, A.Distance);
	}
	else
	{
		float ColorWeight = A.Intersection * DerivedA.Weight * DerivedA.IsConsidered;
		float OpacityWeight = A.Intersection;
	
		Accumulator.Color += A.Color * ColorWeight;
		Accumulator.ColorWeight += ColorWeight;
	
		Accumulator.Opacity += OpacityWeight * DerivedA.Opacity;
		Accumulator.OpacityWeight += OpacityWeight;
	}
}

/** Accumulates center sample. */
void AccumulateCenterSample(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	FGatherSampleDerivedParameters DerivedA = ComputeSampleDerivates(Accumulator, A);
	AccumulateSample(Accumulator, A, DerivedA);
}

void AccumulateCenterSampleAsItsOwnRing(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	AccumulateCenterSample(/* inout */ Accumulator, A);
}

/** Changes the already accumulated weight, used when doing varying gathering sample density. */
void AmendAccumulatedRingsWeight(
	inout FGatherAccumulator Accumulator,
	in FGatherInputParameters NewGatherParameters,
	float AccumulatedWeightMultiplier)
{
	// TODO: UNIMPLEMENTED
}

/** Setup up the accumulator for a ring. */
void StartRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	// NOP
}

/** Accumulates mirror samples. */
void AccumulateMirrorSamples(inout FGatherAccumulator Accumulator, in FGatherSample A, in FGatherSample B)
{
	FGatherSampleDerivedParameters DerivedA = ComputeSampleDerivates(Accumulator, A);
	FGatherSampleDerivedParameters DerivedB = ComputeSampleDerivates(Accumulator, B);

	// Hole filling for foreground pixels.
	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
	{
		if (IsForeground(A.CocRadius) && B.CocRadius > A.CocRadius)
		{
			B.Intersection = max(B.Intersection, A.Intersection);
		}
		else if (IsForeground(B.CocRadius) && A.CocRadius > B.CocRadius)
		{
			A.Intersection = max(B.Intersection, A.Intersection);
		}
	}
	#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
	{
		if (B.CocRadius > A.CocRadius)
		{
			B.Intersection = A.Intersection;
			DerivedB.Weight = DerivedA.Weight;
		}
		else
		{
			A.Intersection = B.Intersection;
			DerivedA.Weight = DerivedB.Weight;
		}
	}
	#else
		#error Unimplemented hole filling.
	#endif

	AccumulateSample(Accumulator, A, DerivedA);
	AccumulateSample(Accumulator, B, DerivedB);
}

/** Digests all samples accumulated for the ring. */
void EndRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	// NOP
}

/** Resolves the background layer output from the accumulator. */
void ResolveAccumulatorOutputs(
	in FGatherAccumulator Accumulator,
	in FGatherResolveInfos GatherResolveInfos,
	out FAccumulatorOutput Output)
{
	// Initializes Output.
	Output = CreateAccumulatorOutput();
	
	float4 NormalizedColor = Accumulator.Color * SafeRcp(Accumulator.ColorWeight);
	float NormalizedOpacity = Accumulator.Opacity * SafeRcp(Accumulator.OpacityWeight);

	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
	if (Accumulator.DistanceToClosestForeground != EXTREMELY_LARGE_COC_RADIUS && Accumulator.ColorWeight > 0)
	{
		float Translucency = Accumulator.DistanceToClosestForeground / Accumulator.GatherParameters.KernelRadius;
		float Opacity = 1 - Translucency;

		Output.ForegroundHoleFillingColor = NormalizedColor * Opacity;
		Output.ForegroundHoleFillingAlpha = Translucency;
	}
	#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
	{
		Output.SlightFocusColor = NormalizedColor;
		Output.SlightFocusOpacity = NormalizedOpacity;
	}
	#else
		#error Unsupported.
	#endif
}


#endif


//------------------------------------------------------- JIMENEZ 2014: CALL OF DUTY WAREFARE.
#if GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_JIMENEZ2014

/** Returns the layer id the sample should go in. (Jiminez 2014, slide 100) */
float ComputeSampleLayer(in FGatherAccumulator Accumulator, in FGatherSample A)
{
	const float CocLayerTransition = 0.5;
	const float CocLayer0Thickness = 0.0;

	float LayerId = saturate((A.CocRadius - Accumulator.GatherParameters.ClosestCocRadius - CocLayer0Thickness) * CocLayerTransition);

	#if CONFIG_ACCUMULATOR_FULL_ALU
		return smoothstep(0.0, 1.0, LayerId);
	#else
		return LayerId;
	#endif
}

float ComputeForegroundFading(in const FGatherInputParameters GatherParameters, float CocRadius)
{
	return saturate((-CocRadius) - (GatherParameters.MaxRecombineAbsCocRadius - 1.0));
}

/** Accumulates a sample. */
void AccumulateSample(inout FGatherAccumulator Accumulator, in FGatherSample A, float Weight)
{
	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
		if (!IsForeground(A.CocRadius))
		{
			A.Intersection = 0.0;
		}
		A.Intersection *= saturate((-A.CocRadius) - (Accumulator.GatherParameters.MaxRecombineAbsCocRadius - 1.0));

	#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
		if (IsForeground(A.CocRadius))
		{
			A.Intersection = 0.0;
		}
		else if (Accumulator.bIsHoleFillingForForeground)
		{
			A.Intersection = 1.0;
		}

	#endif
	
	//A.Intersection *= IsConsideredCocRadius(A.CocRadius);

	float IsInLayer1 = ComputeSampleLayer(Accumulator, A);
	float IsInLayer0 = 1.0  - IsInLayer1;

	Accumulator.Layer[1] += A.Color * (IsInLayer1 * A.Intersection * Weight);
	Accumulator.LayerWeight[1] += IsInLayer1 * A.Intersection * Weight;
	Accumulator.LayerSampleCount[1] += IsInLayer1;

	Accumulator.Layer[0] += A.Color * (IsInLayer0 * A.Intersection * Weight);
	Accumulator.LayerWeight[0] += IsInLayer0 * A.Intersection * Weight;
	Accumulator.LayerSampleCount[0] += IsInLayer0;

	Accumulator.IntersectionSum += A.Intersection;
	Accumulator.ForegroundSum += (IsForeground(A.CocRadius) ? 1.0 : 0.0) * saturate((-A.CocRadius) - (Accumulator.GatherParameters.MaxRecombineAbsCocRadius - 1.0));
}

/** Accumulates center sample. */
void AccumulateCenterSample(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	// Fade intersection as CocRadius becomes close to 0 to fallback to full resolution gathering.	
	A.Intersection *= ComputeForegroundFading(Accumulator.GatherParameters, A.CocRadius);

	// TODO: when accumulating the foreground, there is an issue in geometric edges.
	#if 1
		float WeightA = ComputeSampleWeight(A.CocRadius);
		AccumulateSample(Accumulator, A, WeightA);
		Accumulator.bIsHoleFillingForForeground = IsForeground(A.CocRadius);
		Accumulator.CenterCocRadius = A.CocRadius;
	#endif
}

void AccumulateCenterSampleAsItsOwnRing(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	AccumulateCenterSample(/* inout */ Accumulator, A);
}

/** Changes the already accumulated weight, used when doing varying gathering sample density. */
void AmendAccumulatedRingsWeight(
	inout FGatherAccumulator Accumulator,
	in FGatherInputParameters NewGatherParameters,
	float AccumulatedWeightMultiplier)
{
	// TODO: UNIMPLEMENTED
}

void StartRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	// NOP.
}

/** Accumulates mirror samples. */
void AccumulateMirrorSamples(inout FGatherAccumulator Accumulator, in FGatherSample A, in FGatherSample B)
{	
	float WeightA = ComputeSampleWeight(A.CocRadius);
	float WeightB = ComputeSampleWeight(B.CocRadius);
	
	// Jimenez 2014, slide 70: sample mirroring.
	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
	{
		// Fade intersection as CocRadius becomes close to 0 to fallback to full resolution gathering.	
		A.Intersection *= ComputeForegroundFading(Accumulator.GatherParameters, A.CocRadius);
		B.Intersection *= ComputeForegroundFading(Accumulator.GatherParameters, B.CocRadius);

		// If sample B is behind sample A. CocRadius < 0 when foreground.
		if (B.CocRadius > A.CocRadius)
		{
			B.Intersection = A.Intersection;
			B.CocRadius = A.CocRadius;
			WeightB = WeightA;
		}
		else
		{
			A.Intersection = B.Intersection;
			A.CocRadius = B.CocRadius;
			WeightA = WeightB;
		}
	}
	#endif

	AccumulateSample(Accumulator, A, WeightA);
	AccumulateSample(Accumulator, B, WeightB);
}

void EndRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	// NOP.
}

/** Resolves the foreground layer output from the accumulator. */
void ResolveAccumulatorOutputs(
	in FGatherAccumulator Accumulator,
	in FGatherResolveInfos GatherResolveInfos,
	out FAccumulatorOutput Output)
{
	// Initializes Output.
	Output = CreateAccumulatorOutput();

	// Total number of sample done.
	float TotalSampleCount = GatherResolveInfos.SamplesCount;

	// Compute opacity of layer 0.
	// Jimenez 2014, slide 102.
	// Math verified by Guillaume.
	float KernelWeight = ComputeSampleWeight(Accumulator.GatherParameters.KernelRadius * ((Accumulator.GatherParameters.RingCount + 0.5) / Accumulator.GatherParameters.RingCount));
	float KernelArea = rcp(KernelWeight);
	float Layer0Opacity = saturate(
		(Accumulator.LayerWeight[1] == 0 ? 1 : 0) +
		((1.0 / float(TotalSampleCount)) * KernelArea * Accumulator.LayerWeight[0]));

	#if 1
		Output.ForegroundColor = lerp(
			Accumulator.Layer[1] * SafeRcp(Accumulator.LayerWeight[1]),
			Accumulator.Layer[0] * SafeRcp(Accumulator.LayerWeight[0]),
			Layer0Opacity);

	#else
		Output.ForegroundColor = (Accumulator.Layer[0] + Accumulator.Layer[1]) * SafeRcp(Accumulator.LayerWeight[0] + Accumulator.LayerWeight[1]);

	#endif

	#if 0
		Output.ForegroundAlpha = saturate(Accumulator.IntersectionSum / float(TotalSampleCount));
	
	#else
		// The most reliable way to get a proper alpha channel for foreground is to just count the number of foreground pixel we have found,
		// with the gather, regardless of whether their coc has intersected with the output pixel.
		Output.ForegroundAlpha = Accumulator.ForegroundSum * rcp(float(TotalSampleCount));

		// But it is then technically possible to have found foreground sample, but intersected none of them. Therefor, to avoid
		// dark layout arround out of focus, we force the alpha to be 0 in that specific case.
		Output.ForegroundAlpha = (Accumulator.LayerWeight[0] + Accumulator.LayerWeight[1]) > 0.0 ? Output.ForegroundAlpha : 0.0;
	#endif

	// This are same implementation, and LAYER_PROCESSING_FOREGROUND_AND_BACKGROUND is not supported.
	Output.BackgroundColor = Output.ForegroundColor;

	// To fill with full resolution.
	// HACK: return the sum of the weight because only used for testing > 0.
	Output.BackgroundWeight = Accumulator.LayerWeight[0] + Accumulator.LayerWeight[1];
}

#endif // GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_JIMENEZ2014


//------------------------------------------------------- RING BINNING ACCUMULATOR.
/**
 * Unpublished in house made depth sorting accumulator for background out of focus layer.
 *
 * It is based on the fundamental fact that far rings of the kernel will only intersect as large or larger
 * CocRadius than distance of the ring from the center of ther kernel.
 *
 * The idea is to accumulate the kernel, starting from the largest ring N, to intersect with closer geometry
 * that have a smaller Coc at the end of the accumulation.
 *
 * Considering the samples of ring I < N, there is two buckets:
 *	* previous bucket: that is the accumulation of all samples that have larger Coc than current ring I's
 *      radius, that were found on previously accumulated rings [[N; I[[;
 *  * current bucket: that is the accumulation of all samples that have arround the same Coc as current ring's radius.
 *      Note that it is impossible to find significantly smaller CocRadius in this bucket, because such input sample
 *		would not intersect with the center of the gathering kernel on ring I.
 *
 * For every sample of the ring, it is categorized whether it belongs to previous bucket or current bucket by simply
 * comparing its Coc radius against the average radius of I and I + 1, known as the bucket bordering radius.
 *
 *    (O = center of the kernel)    (A = a sample of ring I)       (B = a sample of ring I + 1)
 *                                                             |
 *       O                   ----[...]----             A       K       B
 *                                                             |
 *                                    (border between current and previous buckets for ring I)
 *
 *    distance(O, K) == the bucket bordering radius.
 *
 * Once all the sample of the ring I has been accumulated, the current bucket get composited into the previous bucket,
 * and get reset to accumulate the next ring I-1, and the bucket bordering radius get updated as well to be between
 * ring I and ring I - 1.
 *
 * The compositing of the current bucket into the previous bucket is simply an hybrid compositing operator between
 * an additive and opacity based blending, that takes into account the following inputs:
 *  * the average Coc radius of the current bucket; 
 *  * the average Coc radius of the previous bucket; 
 *  * the translucency of the current bucket.
 *
 * The idea of this compositing operator is simply to do additive blending when average Coc radii are same, or
 * do opacity based compositing otherwise.
 *
 *
 */
#if GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING

// Because largest rings while intersect only with far away pixels, we processed them first so that lower ring get
// Blended on top if they are closer.
#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
	#define LARGEST_RINGS_FIRST 1
#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
	#define LARGEST_RINGS_FIRST 0
	#error Fix me.
#else
	#error Fix me.
#endif


/** Accumulates sample. */
void AccumulateSample(inout FGatherAccumulator Accumulator, in FGatherSample A, float Weight)
{
	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
		if (IsForeground(A.CocRadius))
		{
			A.Intersection = 0.0;
		}
		// TODO.
		//else if (Accumulator.bIsHoleFillingForForeground)
		//{
		//	A.Intersection = 1.0;
		//}

	#endif

	// Fade intersection if sample's Coc radius is out of the gathering pass coc bounds.
	float IsConsidered = IsConsideredCocRadius(Accumulator.GatherParameters, A.CocRadius);
	
	// Compare the sample's Coc radius with the bucket bordering radius.
	float bBelongsToPrevious = saturate(A.CocRadius - Accumulator.BorderingRadius + 0.5);
	
	#if CONFIG_ACCUMULATOR_FULL_ALU
		bBelongsToPrevious = smoothstep(0.0, 1.0, bBelongsToPrevious);
	#endif

	// First ring is directly accumulated to the current bucket, because merge with potential larger .
	if (Accumulator.bIsFirstRing)
	{
		bBelongsToPrevious = 0.0;
	}

	float bBelongsToCurrent = 1.0 - bBelongsToPrevious;

	#if CONFIG_LAYER_GATHERING
	if (!Accumulator.bIsFirstRing)
	{
		float M = 1;
		M *= 1 - saturate(A.CocRadius - Accumulator.FarBorderingRadius + 0.5);
		M *= saturate(A.CocRadius - Accumulator.CloseBorderingRadius + 0.5);

		bBelongsToCurrent *= saturate(A.CocRadius - Accumulator.CloseBorderingRadius + 0.5);
		bBelongsToPrevious *= 1 - saturate(A.CocRadius - Accumulator.FarBorderingRadius + 0.5);
	}
	#endif

	float CurrentWeight = (A.Intersection * IsConsidered * Weight) * bBelongsToCurrent;

	Accumulator.CurrentColor += A.Color * CurrentWeight;
	Accumulator.CurrentCocRadius += A.CocRadius * CurrentWeight;
	Accumulator.CurrentCocRadiusSquare += A.CocRadius * (A.CocRadius * CurrentWeight);
	Accumulator.CurrentWeight += CurrentWeight;
	
	float PreviousWeight = (A.Intersection * IsConsidered * Weight) * bBelongsToPrevious;

	Accumulator.PreviousColor += A.Color * PreviousWeight;
	Accumulator.PreviousCocRadius += A.CocRadius * PreviousWeight;
	Accumulator.PreviousCocRadiusSquare += A.CocRadius * (A.CocRadius * PreviousWeight);
	Accumulator.PreviousWeight += PreviousWeight;

	// Accumulate current bucket translucency.
	{
		float SampleTranslucency = saturate(A.CocRadius - Accumulator.BorderingRadius);

		//Accumulator.CurrentTranslucency += A.CocRadius > Accumulator.BorderingRadius;
		Accumulator.CurrentTranslucency += SampleTranslucency;
	}
}

/** Merge current bucket into the previous bucket. */
void MergeCurrentBucketIntoPreviousBucket(inout FGatherAccumulator Accumulator, float SampleCount)
{
	// If have not accumulated any sample into the current bucket, just avoid this useless work and the
	// NaNs that would be coming with.
	if (Accumulator.CurrentWeight == 0.0)
	{
		return;
	}

	// Opacity of the current bucket.
	float CurrentRingOpacity = saturate(1 - Accumulator.CurrentTranslucency * rcp(SampleCount));
	
	// Compute current and previous Coc radii.
	float PreviousCocRadius = Accumulator.PreviousCocRadius * rcp(Accumulator.PreviousWeight);
	float CurrentCocRadius = Accumulator.CurrentCocRadius * rcp(Accumulator.CurrentWeight);

	// Whether current bucket is occluding previous bucket.
	// TODO: put a contrast.
	float bOccludingCoc = saturate((PreviousCocRadius - CurrentCocRadius));

	// Compute final factor to use to with previous bucket.
	float PreviousBucketFactor = (Accumulator.PreviousWeight == 0.0) ? 0.0 : (1.0 - CurrentRingOpacity * bOccludingCoc);

	// Compose current ring into previous.
	Accumulator.PreviousColor     = Accumulator.PreviousColor     * PreviousBucketFactor + Accumulator.CurrentColor;
	Accumulator.PreviousCocRadius = Accumulator.PreviousCocRadius * PreviousBucketFactor + Accumulator.CurrentCocRadius;
	Accumulator.PreviousCocRadiusSquare = Accumulator.PreviousCocRadiusSquare * PreviousBucketFactor + Accumulator.CurrentCocRadiusSquare;
	Accumulator.PreviousWeight    = Accumulator.PreviousWeight    * PreviousBucketFactor + Accumulator.CurrentWeight;
}

/** Like mirror samples. */
void AccumulateCenterSample(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	float WeightA = ComputeSampleWeight(A.CocRadius);
	AccumulateSample(Accumulator, A, WeightA);
}

/** Accumulates center sample as it s own ring potentially occluding previous ring instead of between StartRingSamples() and EndRingSamples(). */
void AccumulateCenterSampleAsItsOwnRing(inout FGatherAccumulator Accumulator, in FGatherSample A)
{
	// Error in coc radius error introduced by the under sampling of the kernel
	const float CocRadiusError = CONFIG_COC_RADIUS_CORRECTION ? 1.0 : 0.0;

	// Set the bordering radius at exactly the radius of the center sample.
	Accumulator.BorderingRadius = (0.5 + CocRadiusError) * Accumulator.GatherParameters.KernelRadius * rcp(0.5 + Accumulator.GatherParameters.RingCount);

	// Center sample is first ring only when smallest ring first.
	Accumulator.bIsFirstRing = LARGEST_RINGS_FIRST == 0;

	// Accumulate the center's sample.
	AccumulateCenterSample(Accumulator, A);
	
	MergeCurrentBucketIntoPreviousBucket(Accumulator, 1.0);
}

/** Changes the already accumulated weight, used when doing varying gathering sample density. */
void AmendAccumulatedRingsWeight(
	inout FGatherAccumulator Accumulator,
	in FGatherInputParameters NewGatherParameters,
	float AccumulatedWeightMultiplier)
{
	Accumulator.GatherParameters = NewGatherParameters;
	Accumulator.PreviousColor           *= AccumulatedWeightMultiplier;
	Accumulator.PreviousCocRadius       *= AccumulatedWeightMultiplier;
	Accumulator.PreviousCocRadiusSquare *= AccumulatedWeightMultiplier;
	Accumulator.PreviousWeight          *= AccumulatedWeightMultiplier;
}

/** Setup up the accumulator for a ring. */
void StartRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	Accumulator.bIsFirstRing = RingInfos.bIsFirstRing;
	
	// Error in coc radius error introduced by the under sampling of the kernel
	const float CocRadiusError = CONFIG_COC_RADIUS_CORRECTION ? 1.0 : 0.0;

	Accumulator.BorderingRadius = (RingInfos.RingId + (1.5 + CocRadiusError)) * (Accumulator.GatherParameters.KernelRadius * rcp(0.5 + Accumulator.GatherParameters.RingCount)); 

	#if CONFIG_SGPR_HINTS
	{
		Accumulator.BorderingRadius = ToScalarMemory(Accumulator.BorderingRadius);
	}
	#endif
}

/** Accumulates mirror samples. */
void AccumulateMirrorSamples(inout FGatherAccumulator Accumulator, in FGatherSample A, in FGatherSample B)
{
	// TODO:  mirroring hole filling like Jimenez 2014 on slide 70? Don't think this necessary as it is
	// semantically usefull only for foreground.

	float WeightA = ComputeSampleWeight(A.CocRadius);
	float WeightB = ComputeSampleWeight(B.CocRadius);

	AccumulateSample(Accumulator, A, WeightA);
	AccumulateSample(Accumulator, B, WeightB);
}

/** Digests all samples accumulated for the ring. */
void EndRingSamples(inout FGatherAccumulator Accumulator, in FGatherRingInfos RingInfos)
{
	if (static_condition(RingInfos.bIsFirstRing)) // static_condition because number of ring for loop have UNROLL.
	{
		// Simply move current bucket into previous bucket.
		Accumulator.PreviousColor     = Accumulator.CurrentColor;
		Accumulator.PreviousCocRadius = Accumulator.CurrentCocRadius;
		Accumulator.PreviousCocRadiusSquare = Accumulator.CurrentCocRadiusSquare;
		Accumulator.PreviousWeight    = Accumulator.CurrentWeight;
	}
	else
	{
		MergeCurrentBucketIntoPreviousBucket(Accumulator, RingInfos.SampleCount);
	}

	// Reset current bucket accumulators.
	Accumulator.CurrentColor = 0;
	Accumulator.CurrentCocRadius = 0;
	Accumulator.CurrentCocRadiusSquare = 0;
	Accumulator.CurrentWeight = 0;
	Accumulator.CurrentTranslucency = 0;
}

/** Resolves the background layer output from the accumulator. */
void ResolveAccumulatorOutputs(
	in FGatherAccumulator Accumulator,
	in FGatherResolveInfos GatherResolveInfos,
	out FAccumulatorOutput Output)
{
	// Initializes Output.
	Output = CreateAccumulatorOutput();

	float InvWeight = SafeRcp(Accumulator.PreviousWeight);
	
	// There is a MergeCurrentIntoPrevious() in the center sample, so just need to return normalized
	// previous bucket's color and weight.
	Output.BackgroundColor = Accumulator.PreviousColor * InvWeight;
	Output.BackgroundWeight = Accumulator.PreviousWeight;
		
	Output.BackgroundCocAvgAndSquareAvg.x = Accumulator.PreviousCocRadius * InvWeight;
	Output.BackgroundCocAvgAndSquareAvg.y = Accumulator.PreviousCocRadiusSquare * InvWeight;
}

#endif // GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING
