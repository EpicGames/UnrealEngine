// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFGatherCommon.usf: common code shared by other gather USH files
	for DOFGatherPass.usf. 
=============================================================================*/

#pragma once

#include "DOFCommon.ush"
#include "DOFCocTileCommon.ush"


//------------------------------------------------------- DEBUG COMPILE TIME CONFIG

// To iterate faster on the gathering kernel, reduce the number of sample to minimum
// to get the shader compiler building quicker regardless of the permutation.
#define DEBUG_FORCE_LOW_RING_COUNT 0
#define DEBUG_NO_FAST_ACCUMULATOR 0

// When on, show FFastAccumulator in green, versus FGatherAccumulator in red.
#define DEBUG_FAST_ACCUMULATOR 0

// Force to disable random offset of the gathering kernel
#define DEBUG_NO_RANDOM_OFFSET 0

// Force to sample mip level that have same resolution as output.
#define DEBUG_FORCE_GATHER_HIGHEST_MIP_LEVEL 0


//------------------------------------------------------- DIMENSIONS VALUES.

// Matching C++'s FRCPassDiaphragmDOFGather::EQualityConfig
#define GATHER_QUALITY_LOW_QUALITY 0
#define GATHER_QUALITY_HIGH_QUALITY 1
#define GATHER_QUALITY_SCATTER_OCCLUSION_OUTPUT 2
#define GATHER_QUALITY_CINEMATIC 3


//------------------------------------------------------- ENUM VALUES

/** Different gathering accumulator implementation to be used. */
	// In house: simple one bucket accumulator for slight out of focus and separate foreground hole filling.
	#define GATHER_ACCUMULATOR_METHOD_SIMPLE 0

	// JIMENEZ SIGGRAPH 2014: CALL OF DUTY WAREFARE: 2 layers only, closest layer defined by tiled Coc.
	#define GATHER_ACCUMULATOR_METHOD_JIMENEZ2014 1

	// In house: each sampling ring is a bin.
	#define GATHER_ACCUMULATOR_METHOD_RINGBINNING 2


/** Layouts of the kernel's samples to use. */
	// Disk (or custom area if simulating a bokeh with a LUT) that scales with kernel radius.
	#define KERNEL_LAYOUT_SCALABLE_DISK 0
	
	// Non scalable square shaped kernel that have all samples aligned with input samples.
	#define KERNEL_LAYOUT_ALIGNED_SQUARE 1
	

/** Kernel sampling density compile time mode. */
	// The kernel sample density is constent
	#define KERNEL_DENSITY_CONSTANT 0
	#define KERNEL_DENSITY_CONSTANT_HEXAWEB 1

	// The kernel offer a run time toggable between:
	//  - constant sample density;
	//  - donut area with same density as if was a fully constant density, and a higher density disk in its center.
	#define KERNEL_DENSITY_HIGHER_IN_CENTER_DISK 2

	// The kernel is hexagonal web, that have 1/4 sampling density everywhere but the last ring, particularily useful
	// for fast gathering on hardware that can particularily be texture fetch bound.
	#define KERNEL_DENSITY_LOWER_IN_CENTER_HEXAWEB 3

	// Kernel that does multiple passes of entire disks at different mip levels.
	#define KERNEL_DENSITY_RECURSIVE_DISKS 4

	// Kernel is constant, but with varying number of rings
	#define KERNEL_DENSITY_CONSTANT_DYNAMIC_RING_COUNT 5


/** Technic to use with mip level. */
// non conservative with interleaved offset.
#define MIP_LEVEL_METHOD_INTERLEAVED_OFFSET 0

// 1: conservative.
#define MIP_LEVEL_METHOD_CONSERVATIVE_NO_RANDOM 1


//------------------------------------------------------- COMPILE TIME CONFIG

// Configures the layer processing.
#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
	#define GATHER_ACCUMULATOR_METHOD (GATHER_ACCUMULATOR_METHOD_JIMENEZ2014)
	#define CONFIG_ENABLE_FAST_ACCUMULATOR 1

#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
	#define GATHER_ACCUMULATOR_METHOD (GATHER_ACCUMULATOR_METHOD_SIMPLE)

#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
	#define GATHER_ACCUMULATOR_METHOD (GATHER_ACCUMULATOR_METHOD_RINGBINNING)
	#define CONFIG_ENABLE_FAST_ACCUMULATOR 1

	// Mirror the bokeh.
	#define CONFIG_POINT_MIRROR_BOKEH -1

	// Background DOF can suffer under sampling on smaller out of focus, because
	// the gathering kernel is too large.
	#define CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER 1
	
	// Background need coc radius error correction to reduce amount of background leaking.
	#define CONFIG_COC_RADIUS_CORRECTION 1

#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
	#define GATHER_ACCUMULATOR_METHOD (GATHER_ACCUMULATOR_METHOD_SIMPLE)

	// Slight out of focus just gather all samples.
	#define CONFIG_KERNEL_LAYOUT (KERNEL_LAYOUT_ALIGNED_SQUARE)

#else
	#error Unknown layer processing.

#endif

// Configures the quality settings.
#if DIM_GATHER_QUALITY == GATHER_QUALITY_LOW_QUALITY
	#define CONFIG_ACCUMULATOR_FULL_ALU 0
	#define CONFIG_FAST_ACCUMULATOR_RING_COUNT ((DIM_GATHER_RING_COUNT) - 1)
	#define CONFIG_FAST_ACCUMULATOR_KERNEL_DENSITY (KERNEL_DENSITY_CONSTANT_HEXAWEB)
	//#define CONFIG_FAST_ACCUMULATOR_KERNEL_DENSITY (KERNEL_DENSITY_HEXAWEB_LOWER_IN_CENTER)

	// Highlights needs a bit of help.
	#define CONFIG_SLIGHT_RING_ROTATION 1

#elif DIM_GATHER_QUALITY == GATHER_QUALITY_HIGH_QUALITY
	#define CONFIG_WEIGHT_SAMPLE_FROM_COC 1
	#define CONFIG_MIRRORED_SAMPLER 1
	#define CONFIG_PETZVAL 1

#elif DIM_GATHER_QUALITY == GATHER_QUALITY_SCATTER_OCCLUSION_OUTPUT
	#define CONFIG_WEIGHT_SAMPLE_FROM_COC 1
	#define CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER 1
	#define CONFIG_MIRRORED_SAMPLER 1
	#define CONFIG_PETZVAL 1

#elif DIM_GATHER_QUALITY == GATHER_QUALITY_CINEMATIC
	#define CONFIG_WEIGHT_SAMPLE_FROM_COC 1
	#define CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER 1
	#define CONFIG_MIRRORED_SAMPLER 1
	#define CONFIG_LAYER_GATHERING 1
	#define CONFIG_PETZVAL 1

#else
	#error Unknon gathering quality.
#endif

// Configures the layout accepted has input.
#if CONFIG_DOF_ALPHA
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_ALPHA_COC)
#elif DIM_RGB_COLOR_BUFFER
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC)
#else
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_COC)
#endif


//------------------------------------------------------- DEFAULT CONFIG

// Allows to branch at run time to use the fast accumulator to reduce VALU when coc
// range in neighborhood is know to be small.
#ifndef CONFIG_ENABLE_FAST_ACCUMULATOR
	#define CONFIG_ENABLE_FAST_ACCUMULATOR 0
#endif

// Defines the kernel density to use for the fast accumulator.
#ifndef CONFIG_FAST_ACCUMULATOR_KERNEL_DENSITY
	#define CONFIG_FAST_ACCUMULATOR_KERNEL_DENSITY (KERNEL_DENSITY_CONSTANT)
#endif

// Defines the number of ring for fast accumulator.
// It is good to lower this on texture fetch bound hardware, so that the texture fetch budget can be
// fore accumulator that requires more sample because of the point sampling used.
#ifndef CONFIG_FAST_ACCUMULATOR_RING_COUNT
	#define CONFIG_FAST_ACCUMULATOR_RING_COUNT (DIM_GATHER_RING_COUNT)
#endif

// Selects the code to drive the kernel sampling density.
#ifndef CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER
	#define CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER 0
#endif

// Technic to use with mip level.
#ifndef CONFIG_MIP_LEVEL_METHOD
	#define CONFIG_MIP_LEVEL_METHOD (MIP_LEVEL_METHOD_INTERLEAVED_OFFSET)
#endif

// Kernel sample layout to use for gathering.
#ifndef CONFIG_KERNEL_LAYOUT
	#define CONFIG_KERNEL_LAYOUT (KERNEL_LAYOUT_SCALABLE_DISK)
#endif

// Give hint to the compiler about what can be scalarized.
#define CONFIG_SGPR_HINTS 1

// Clamp the input buffers' UV to avoid sampling outside viewport.
#define CONFIG_CLAMP_INPUT_BUFFER_UV 1

// Whether the samples should be weighted according to their Coc.
#ifndef CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER
	#define CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER 0
#endif

// Whether the samples should be weighted according to their Coc.
#ifndef CONFIG_WEIGHT_SAMPLE_FROM_COC
	#define CONFIG_WEIGHT_SAMPLE_FROM_COC 0
#endif

// Whether to allow accumulator do its highest ALU computation.
#ifndef CONFIG_ACCUMULATOR_FULL_ALU
	#define CONFIG_ACCUMULATOR_FULL_ALU 1
#endif

// Whether to point mirror the bokeh.
#ifndef CONFIG_POINT_MIRROR_BOKEH
	#define CONFIG_POINT_MIRROR_BOKEH 1
#endif

// Uses mirrored sampler when gathering outside viewport rect.
#ifndef CONFIG_MIRRORED_SAMPLER
	#define CONFIG_MIRRORED_SAMPLER 0
#endif

// Substract the CocRadius error to the sample distance to fight against layering opacity issues caused by undersampling.
#ifndef CONFIG_COC_RADIUS_CORRECTION
	#define CONFIG_COC_RADIUS_CORRECTION 0
#endif

/** Defines the maximum number of gathering sampling density changes supported in the kernel. */
#ifndef CONFIG_MAX_SAMPLING_DENSITY_CHANGES
	#define CONFIG_MAX_SAMPLING_DENSITY_CHANGES 3
#endif

/** Adds a slight rotation to the samples to reduce interferance sample location and texel grid with the point sampler,
 * that improves highlight post filtering. */
#ifndef CONFIG_SLIGHT_RING_ROTATION
	#define CONFIG_SLIGHT_RING_ROTATION 0
#endif

/** Whether the gathering is done in multiple layer over the kernel. */
#ifndef CONFIG_LAYER_GATHERING
	#define CONFIG_LAYER_GATHERING 0
#endif


//------------------------------------------------------- CONFIG CHECKS

#ifndef GATHER_ACCUMULATOR_METHOD
	#error You at the very least need to choose the an accumulator.
#endif

#ifndef CONFIG_GATHER_INPUT_LAYOUT
	#error You at the very least need to choose the gather input layout.
#endif

//------------------------------------------------------- CONFIG DERIVED VALUES

// Wether the group size is larger than COC tile size.
#define GROUP_SIZE_IS_LARGER_THAN_COC_TILE_SIZE 0

//------------------------------------------------------- STAND ALONE FUNCTIONS

// Compute sample weight according's to its Coc radius.
float ComputeSampleWeight(float CocRadius)
{
	#if CONFIG_WEIGHT_SAMPLE_FROM_COC
		const float PixelRadius = 0.25; // = 0.5 / CocDownresSample;
		return min(rcp(PI * CocRadius * CocRadius), rcp(PI * PixelRadius * PixelRadius));

	#else
		return 1.0;
	#endif
}


//------------------------------------------------------- STRUCTS

/** Gather input generally from larger gathering radius. */
struct FLargerConvolution
{
	/** Gathered color. */
	float4 Color;

	/** Weight of the color. */
	float Weight;
	
	/** Coc average and squared average. */
	float2 CocAvgAndSquareAvg;
};

/** Gathering parameters. */
struct FGatherInputParameters
{
	// Compile time maximum number of rings to gather.
	uint MaxRingCount;
	
	// Runtime number of rings to gather <= MaxRingCount.
	uint RingCount;

	// buffer UV coordinate of the center of the kernel to gather.
	float2 InputBufferCenterUV;
	
	float2 ViewportUV;

	// Multiplication factors to map DispatchThreadID -> InputBufferUV.
	float2 DispatchThreadIdToInputBufferUV;

	// Radius of the kernel at half res pixels.
	float KernelRadius;

	// Random signals
	float Random[2];

	// Whether going to use accumulator or fast accumulator.
	bool bFastGathering;

	// The Coc radius that is the closest to the camera.
	float ClosestCocRadius;

	// Saturated affine transformation.
	float2 ConsiderCocRadiusAffineTransformation0;
	float2 ConsiderCocRadiusAffineTransformation1;
	float2 ConsiderAbsCocRadiusAffineTransformation;

	// TODO: clean that with CocRadiusMinMaxBoundaries
	float MaxRecombineAbsCocRadius;

	// Sampling density compile time mode. KERNEL_DENSITY_*
	uint KernelSamplingDensityMode;

	// Compile time number of rings for density changes.
	uint DensityChangeRingCount;

	// Compile time max number of density changes allowed.
	uint MaxDensityChangeCount;
	
	// Tile constant about what is the smallest intersectable radius.
	float MinIntersectableCocRadiusAbs;
};

/** Informations about a ring. */
struct FGatherRingInfos
{
	/** Id of the ring. 0 is the smallest ring after the center sample. */
	uint RingId;

	/** Total number of samples on the ring. */
	uint SampleCount;

	/** Compile time constant on whether this is the first accumualted ring. */
	bool bIsFirstRing;

	// Total weight of the ring.
	float Weight;

	/** Radius of the ring in pixels from the center of the kernel. */
	float Radius;
};

/** Informations about the kernel when the accumulator resolves. */
struct FGatherResolveInfos
{
	// Number of accumulated samples.
	// This is a float because of density changes that can happen with AmendAccumulatedRingsWeight().
	float SamplesCount;
};

/** Structs that holds data about a sample for gathering. */
struct FGatherSample
{
	// Sample's scene color (and optionally alpha channel).
	float4 Color;

	// Sample's radius of the Coc
	float CocRadius;

	// Sample distance from center of kernel, in CocRadius unit.
	float Distance;

	// Sample's intersection.
	float Intersection;
};

/** Gathering accumulator output. */
struct FAccumulatorOutput
{
	// Foreground color.
	float4 ForegroundColor;

	// Foreground alpha channel to use to compose on in focus and background.
	float ForegroundAlpha;
	
	// Foreground hole filling color and alpha.
	float4 ForegroundHoleFillingColor;
	float ForegroundHoleFillingAlpha;
	
	// Background color.
	float4 BackgroundColor;

	// Weight of the backrgound.
	float BackgroundWeight;
	
	// Coc average and variance for background.
	float2 BackgroundCocAvgAndSquareAvg;

	// Slight out of focus color and opacity.
	float4 SlightFocusColor;
	float SlightFocusOpacity;
};


//------------------------------------------------------- METHODS

// Returns whether this Coc radius is considered or not for this pass. If not considered, it means this Coc radius will
// be gathered at higher or lower resolution gathering pass.
float IsConsideredCocRadius(in const FGatherInputParameters GatherParameters, float SampleCocRadius)
{
	// Hint: This might look expensive, but this is just for framework flexibility: most of the
	// AffineTransformation are actually set so that SaturateWithAffineTransformation always returns
	// 1 at compile time.
	return (
		SaturateWithAffineTransformation(SampleCocRadius, GatherParameters.ConsiderCocRadiusAffineTransformation0) *
		SaturateWithAffineTransformation(SampleCocRadius, GatherParameters.ConsiderCocRadiusAffineTransformation1) *
		SaturateWithAffineTransformation(abs(SampleCocRadius), GatherParameters.ConsiderAbsCocRadiusAffineTransformation));
}


FAccumulatorOutput CreateAccumulatorOutput()
{
	FAccumulatorOutput AccumulatorOutput;
	AccumulatorOutput.ForegroundColor = 0;
	AccumulatorOutput.ForegroundAlpha = 0;
	AccumulatorOutput.ForegroundHoleFillingColor = 0;
	AccumulatorOutput.ForegroundHoleFillingAlpha = 1;
	AccumulatorOutput.BackgroundColor = 0;
	AccumulatorOutput.BackgroundWeight = 0;
	AccumulatorOutput.BackgroundCocAvgAndSquareAvg = 0;
	AccumulatorOutput.SlightFocusColor = 0;
	AccumulatorOutput.SlightFocusOpacity = 0;
	return AccumulatorOutput;
}
