// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFGatherKernel.usf: gather sampling kernel of Diaphragm DOF. 
=============================================================================*/

#pragma once

#include "DOFGatherCommon.ush"
#include "DOFGatherAccumulator.ush"


//------------------------------------------------------- PARAMETERS

float4 ViewportSize;
float4 GatherInputSize;
float2 GatherInputViewportSize;
Texture2D GatherInput_SceneColor;
Texture2D GatherInput_SeparateCoc;

Texture2D BokehLUT;

float Petzval;
float PetzvalFalloffPower;
float2 PetzvalExclusionBoxExtents;
float PetzvalExclusionBoxRadius;

//------------------------------------------------------- GATHERING KERNEL

// LARGEST_RINGS_FIRST controles the order the rings must be proceded.
#ifndef LARGEST_RINGS_FIRST
	#define LARGEST_RINGS_FIRST 0
#endif


// Returns the position of the sample on the unit circle (radius = 1) for a given ring.
float2 GetDiskSampleOnUnitCirle(uint RingId, uint RingSampleIteration, uint RingSampleId)
{
	float SampleRingPos = RingSampleId;

	// Do not allign all j == 0 samples of the different ring on the X axis to increase minimal distance between all
	// samples, that reduce variance to clean by post filtering.
	#if 1
		SampleRingPos += (RingId - 2 * (RingId / 2)) * 0.5;
	#endif

	#if CONFIG_SLIGHT_RING_ROTATION
		SampleRingPos += (RingId + 1) * 0.2;
	#endif

	float SampleAngle = PI * SampleRingPos / float(RingSampleIteration);
			
	return float2(cos(SampleAngle), sin(SampleAngle));
}

// Returns the rotation matrix to use between sample of the ring.
float2x2 GetSampleRotationMatrix(uint RingSampleIteration)
{
	float RotationAngle = PI / float(RingSampleIteration);

	float C = cos(RotationAngle);
	float S = sin(RotationAngle);

	return float2x2(
		float2( C,  S),
		float2(-S,  C));
}


struct FFastAccumulatorOutput
{
	// Accumulated Color.
	float4 Color;
};

// Amends the weight of the previously accumulated samples in the fast accumulator.
void AmendAccumulatedRingsWeight(inout FFastAccumulatorOutput FastAccumulator, const float AccumulatedWeightMultiplier)
{
	FastAccumulator.Color *= AccumulatedWeightMultiplier;
}

// Shared scope with the different functions of the kernel.
struct FPrivateKernelSharedScope
{
	// ------- compile time constant category.

	// sampler to use.
	bool bBilinearSampleInput;

	// Whether the kernel has enough samples enought sample to do the entire convolution (use by slight out of focus).
	bool bKenrelHasEnoughSamples;
	
	
	// ------- run time category.

	// UV coordinate of the kernel center in GatherInput.
	float2 KernelCenterUV;

	// Multiplier to transform sample count unit to sample position in pixels of GatherInput's mip level 0.
	// Set at compile time to 1 when bKenrelHasEnoughSamples==true.
	float SampleCountToSamplePosition;
	
	// Sampling mip level for GatherInput.
	float GatherInputMipLevel;

	// GatherInput's max UVs.
	// TODO: this is currently MipLevel  depent, but could save some VGPR by making it independent.
	float2 GatherInputMaxUV;

	// Multiplier to apply when computing sample intersection. Higher value sharpen, lower value feather.
	float IntersectionMultiplier;

	// Used to stretch sample positions for petzval lens effect
	float2x2 PetzvalTransform;
};


// Computes intersection of sample from its Coc and its distance.
// Returns 0: no intersection, 1: intersection.
float ComputeSampleIntersection(FPrivateKernelSharedScope KernelScope, float SampleCocRadius, float SampleDistance)
{
	// Intersection of output pixel's disk area with sample's Coc.
	#if 1
		// To have an intersection of 50% when the Sample's Coc radius == SampleDistance to be area conservative
		// with bilinear sampled gathering or hybrid scattering.
		const float Offset = 0.5;

		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
			// Uses a sharper intersection for slight out of focus like Recombine pass since it is being used
			// for full res temporal statbility clamping box, so the aliasing this may introduce at half res is
			// Fine.
			const float Multiplier = 4.0;
		#else
			const float Multiplier = KernelScope.IntersectionMultiplier;
		#endif

		return saturate((abs(SampleCocRadius) - SampleDistance) * Multiplier + Offset);

	// Same as Circle DOF.
	#elif 0
		// 0.5 because Coc radius is at half res, and 0.5 so that the fallof is only the radius of a ful res pixel.
		const float IntersectionSharpen = 0.5 * 0.5;

		// TODO: missing +0.5 to be energy conservative.
		return saturate((abs(SampleCocRadius) - SampleDistance) * IntersectionSharpen);

	#else // Hard cut of.
		if (SampleDistance > abs(SampleCocRadius))
		{
			return 0.0;
		}
		return 1.0;

	#endif
}


// Returns the total number of sampling iteration for a given ring id.
uint GetRingSamplingPairCount(const uint KernelSamplingDensityMode, uint RingId)
{
	if (static_condition(KernelSamplingDensityMode == KERNEL_DENSITY_CONSTANT_HEXAWEB) ||
		static_condition(KernelSamplingDensityMode == KERNEL_DENSITY_LOWER_IN_CENTER_HEXAWEB))
	{
		return (RingId + 1) * 3;
	}

	// This number of sample is carefully chosen to have exact number of sample a square shaped ring (SquarePos).
	return (RingId + 1) * 4;
}

// Returns the total number of sample of the kernel.
uint GetKernelSampleCount(const uint KernelSamplingDensityMode, uint RingCount)
{
	if (static_condition(KernelSamplingDensityMode == KERNEL_DENSITY_CONSTANT_HEXAWEB) ||
		static_condition(KernelSamplingDensityMode == KERNEL_DENSITY_LOWER_IN_CENTER_HEXAWEB))
	{
		return 1 + 3 * RingCount * (RingCount + 1);
	}

	// Depends on GetRingSamplingPairCount().
	return 1 + 4 * RingCount * (RingCount + 1);
}


// Fetch and unpack a sample from GatherInput.
FGatherSample FetchGatherSample(in FPrivateKernelSharedScope KernelScope, float2 SampleUV)
{
	#if CONFIG_MIRRORED_SAMPLER
	{
		SampleUV = max(SampleUV, -SampleUV);
		#if CONFIG_CLAMP_INPUT_BUFFER_UV
			SampleUV = min(SampleUV, 2.0 * KernelScope.GatherInputMaxUV - SampleUV);
		#else
			SampleUV = min(SampleUV, 2.0 - SampleUV);
		#endif
	}
	#elif CONFIG_CLAMP_INPUT_BUFFER_UV
	{
		SampleUV = min(SampleUV, KernelScope.GatherInputMaxUV);
	}
	#endif

	SampleUV = min(SampleUV, KernelScope.GatherInputMaxUV);

	float4 RawSample0 = 0;
	float4 RawSample1 = 0;

	if (KernelScope.bBilinearSampleInput)
	{
		RawSample0 = GatherInput_SceneColor.SampleLevel(
			GlobalBilinearClampedSampler, SampleUV, KernelScope.GatherInputMipLevel);
		RawSample1 = GatherInput_SeparateCoc.SampleLevel(
			GlobalBilinearClampedSampler, SampleUV, KernelScope.GatherInputMipLevel);
	}
	else
	{
		RawSample0 = GatherInput_SceneColor.SampleLevel(
			GlobalPointClampedSampler, SampleUV, KernelScope.GatherInputMipLevel);
		RawSample1 = GatherInput_SeparateCoc.SampleLevel(
			GlobalPointClampedSampler, SampleUV, KernelScope.GatherInputMipLevel);
	}

	FGatherSample Sample;
	#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_COC
	{
		Sample.Color = float4(RawSample0.rgb, 1);
		Sample.CocRadius = RawSample0.a;
	}
	#elif CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC
	{
		Sample.Color = RawSample0;
		Sample.CocRadius = RawSample1.r;
	}
	#elif CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC
	{
		Sample.Color = float4(RawSample0.rgb, 1);
		Sample.CocRadius = RawSample1.r;
	}
	#else
		#error Unknown gather input layout.
	#endif

	Sample.Distance = 0.0;
	Sample.Intersection = 1.0;
	return Sample;
}


// Update run time kernel scope's members from compile time one, and gather parameters.
void UpdateRuntimeKernelScope(
	in FGatherInputParameters GatherParameters,
	inout FPrivateKernelSharedScope KernelScope)
{
	// Radius of a sample in the unit circle.
	const float UnitSampleRadius = rcp(GatherParameters.RingCount + 0.5);

	// Compute the mip level to sample.
	#if CONFIG_MIP_LEVEL_METHOD == MIP_LEVEL_METHOD_INTERLEAVED_OFFSET	
		// Compute random uniformly distributed on disk (centered, diameter = 1).
		float2 RandomCircle = (0.48 * sqrt(GatherParameters.Random[0])) * float2(cos(2 * PI * GatherParameters.Random[1]), sin(2 * PI * GatherParameters.Random[1]));

		float MipLevel = floor(0.5 + log2(GatherParameters.KernelRadius * UnitSampleRadius));

	#elif CONFIG_MIP_LEVEL_METHOD == MIP_LEVEL_METHOD_CONSERVATIVE_NO_RANDOM
		float2 RandomCircle = 0;

		float MipLevel = ceil(log2(GatherParameters.KernelRadius * UnitSampleRadius));

	#else
		#error Unknown mip level technic.

	#endif
	
	// DEBUG: disable random offset.
	#if DEBUG_NO_RANDOM_OFFSET
		RandomCircle = 0;
	#endif

	// DEBUG: force to sample mip level that have same resolution as output.
	#if DEBUG_FORCE_GATHER_HIGHEST_MIP_LEVEL
		MipLevel = 0;
	#endif

	// When there is enough samples, can sample the MipLevel0 just fine.
	if (static_condition(KernelScope.bKenrelHasEnoughSamples))
	{
		MipLevel = 0;
	}

	// Compute the randomising range of the kernel offset in pixels.
	float KernelRandomRange = GatherParameters.KernelRadius * UnitSampleRadius;
	if (static_condition(KernelScope.bBilinearSampleInput))
	{
		KernelRandomRange = max(GatherParameters.KernelRadius - GatherParameters.MaxRingCount - pow(2.0, uint(MipLevel)), 0.0) * UnitSampleRadius;
	}

	// No need to do kernel randomization when there is enough samples.
	if (static_condition(KernelScope.bKenrelHasEnoughSamples))
	{
		KernelRandomRange = 0;
	}

	KernelScope.KernelCenterUV = GatherParameters.InputBufferCenterUV + (RandomCircle * KernelRandomRange) * GatherInputSize.zw;

	KernelScope.PetzvalTransform = 0;
	if (Petzval != 0)
	{
		const float2 BokehCenter = GatherParameters.ViewportUV * 2.0 - 1.0;
		
		const float Radius = PetzvalExclusionBoxRadius * min(PetzvalExclusionBoxExtents.x, PetzvalExclusionBoxExtents.y);
		const float2 BoxToBokeh = sign(BokehCenter) * max(abs(BokehCenter) - PetzvalExclusionBoxExtents + Radius, 0.0);
		float BokehDistance = max(0.0, length(BoxToBokeh));
		const float2 Normal = BoxToBokeh / BokehDistance;
		BokehDistance = max(0.0, BokehDistance - Radius);

		if (BokehDistance > 0)
		{
			const float PetzvalIntensity = pow(BokehDistance, PetzvalFalloffPower);

			const float PetzvalStretchFactor = rcp(1 + abs(Petzval) * PetzvalIntensity);

			const float2 Tangent = float2(Normal.y, -Normal.x);
			const float2x2 ToSagittalSpace = {Normal.x, Normal.y, Tangent.x, Tangent.y};
			bool bMask = Petzval > 0;
			const float2x2 Scale = { 
				bMask ? PetzvalStretchFactor : 1.0, // Sagittal
				0.0, 0.0, 
				!bMask ? PetzvalStretchFactor : 1.0 // Tangential
			};
			KernelScope.PetzvalTransform = mul(mul(ToSagittalSpace, Scale), transpose(ToSagittalSpace));
		}
		else
		{
			const float2x2 Identity = { 1, 0, 0, 1 };
			KernelScope.PetzvalTransform = Identity;
		}
	}

	KernelScope.SampleCountToSamplePosition = GatherParameters.KernelRadius * UnitSampleRadius;

	float MipTexelSize = pow(2.0, uint(MipLevel));
	KernelScope.GatherInputMaxUV = (GatherInputViewportSize.xy - 0.5 * MipTexelSize) * GatherInputSize.zw;
	KernelScope.GatherInputMipLevel = MipLevel;
	KernelScope.IntersectionMultiplier = rcp(MipTexelSize);
		
	// When there is enough samples, the sample count measurement and the sample position measurement are same.
	if (static_condition(KernelScope.bKenrelHasEnoughSamples))
	{
		KernelScope.SampleCountToSamplePosition = 1;
	}

	// Gives hints to the compiler that these var can be shifted to SGPR.
	#if CONFIG_SGPR_HINTS
	{
		KernelScope.SampleCountToSamplePosition = ToScalarMemory(KernelScope.SampleCountToSamplePosition);
		KernelScope.GatherInputMaxUV = ToScalarMemory(KernelScope.GatherInputMaxUV);
		KernelScope.GatherInputMipLevel = ToScalarMemory(KernelScope.GatherInputMipLevel);
		KernelScope.IntersectionMultiplier = ToScalarMemory(KernelScope.IntersectionMultiplier);
	}
	#endif
}

// Apply approximate Petzval bokeh lens effect.
// A more precise version would calculate the transform per sample, rather than per kernel.
// This version assumes the bokeh that are gathered are all centered on the kernel center, rather than on the sample center.
// This is incorrect and can cause bokeh to stretch into a 'banana shape', rather than oval.
// But it's significantly faster and usually not noticeable (more prominent bokeh are scattered anyway).
float2 ApplyApproxPetzval(in FPrivateKernelSharedScope KernelScope, float2 SampleUV)
{
#if CONFIG_PETZVAL
	if (Petzval != 0)
	{
		return mul(SampleUV, KernelScope.PetzvalTransform);
	}
	else
#endif //CONFIG_PETZVAL
	{
		return SampleUV;
	}
}

// Transform at compile time a 2 dimensional batch's constant into sample pair constant, by using rotation invariance.
float2 SampleConstFromBatchConst(const uint BatchSampleId, float2 BatchConst)
{
	/**
	 *             Y
	 *             ^
	 *             |
	 *        1    |
	 *             |
	 *             |       0
	 *             |
	 * - - - - - - O - - - - > X
	 */
	if (BatchSampleId == 1)
		return float2(-BatchConst.y, BatchConst.x);
	return BatchConst;
}


// Gather a ring into the accumulator.
void GatherRingSamples(
	in FGatherInputParameters GatherParameters,
	in FPrivateKernelSharedScope KernelScope,
	in uint RingId,
	in bool bIsFirstRing,
	inout FGatherAccumulator Accumulator,
	inout FFastAccumulatorOutput FastAccumulator)
{
	// Number of sample iteration for this ring.
	const uint RingSamplePairCount = GetRingSamplingPairCount(GatherParameters.KernelSamplingDensityMode, RingId);

	// Number of sample pair to process per batch.
	// TODO: Could potentially do 4 using symetries? Might be unpracticable because of VGPR pressure. 
	const uint SamplePairBatchSize = (RingSamplePairCount % 2) ? 1 : 2;

	// Number of batch to process.
	const uint BatchCount = RingSamplePairCount / SamplePairBatchSize;

	// Distance of the ring from the center of the kernel in sample count.
	const uint RingDistance = uint(RingId + 1);

	float RingRadius = ToScalarMemory(RingDistance * KernelScope.SampleCountToSamplePosition);

	#if CONFIG_SGPR_HINTS
	{
		GatherParameters.KernelRadius = ToScalarMemory(GatherParameters.KernelRadius);
	}
	#endif

	// Generate at compile time sample rotation matrix.
	const float2x2 SampleRotationMatrix = GetSampleRotationMatrix(RingSamplePairCount);

	// Generates at compile time first sample location on circle (radius = 1).
	const float2 FirstCircleUnitPos = GetDiskSampleOnUnitCirle(RingId, RingSamplePairCount, /* BatchId = */ 0);

	// Position of the first sample on circle with radius according to KernelRadius.
	float2 FirstCircleSamplePosOffset = FirstCircleUnitPos * RingRadius;

	// Setup iteratable SGPR
	float2 CurrentCircleUnitPos = FirstCircleUnitPos;
	float2 CurrentCircleSamplePosOffset = FirstCircleSamplePosOffset;

	#if CONFIG_SGPR_HINTS
	{
		CurrentCircleUnitPos = ToScalarMemory(CurrentCircleUnitPos);
		CurrentCircleSamplePosOffset = ToScalarMemory(CurrentCircleSamplePosOffset);
	}
	#endif

	// Iterate over the batch of samples.
	for (uint BatchId = 0; BatchId < BatchCount; BatchId++)
	#if defined(WORKARROUND_UE_58394)
		WORKARROUND_UE_58394 // TODO: delete once original issue is fixed.
	#endif
	{
		// Rotate the samples position along the ring.
		CurrentCircleUnitPos = mul(CurrentCircleUnitPos, SampleRotationMatrix);
		CurrentCircleSamplePosOffset = mul(CurrentCircleSamplePosOffset, SampleRotationMatrix);

		#if CONFIG_SGPR_HINTS
		{
			CurrentCircleUnitPos = ToScalarMemory(CurrentCircleUnitPos);
			CurrentCircleSamplePosOffset = ToScalarMemory(CurrentCircleSamplePosOffset);
		}
		#endif

		float2 BatchCirleUnitPos = CurrentCircleUnitPos;
		float2 BatchCircleSamplePosOffset = CurrentCircleSamplePosOffset;

		// Generates at compile time sample location if square shaped ring in sample count distance from the kernel's center.
		//
		// RingId = 0:
		//      Y
		//      |
		//    3 2 1
		//    4   0 -X
		//
		// RingId = 1:
		//      Y
		//      |
		//  6 5 4 3 2
		//  7       1
		//  8       0 -X
		//
		// ...
		float2 BatchSquarePos;
		if (BatchId < RingDistance)
		{
			BatchSquarePos.x = RingDistance;
			BatchSquarePos.y = BatchId;
		}
		else if (BatchId < RingDistance * 3)
		{
			BatchSquarePos.x = float(RingDistance) - float(BatchId - RingDistance);
			BatchSquarePos.y = RingDistance;
		}
		else
		{
			BatchSquarePos.x = -float(RingDistance);
			BatchSquarePos.y = RingDistance - (BatchId - RingDistance * 3);
		}

		// Iterate over the pair of sample of the batch to share vgpr setup and rotating ALU
		for (uint BatchSampleId = 0; BatchSampleId < SamplePairBatchSize; BatchSampleId++)
		{
			// Swizzle the batch positions' X and Y components.
			float2 CirleUnitPos = SampleConstFromBatchConst(BatchSampleId, BatchCirleUnitPos);
			const float2 SquarePos = SampleConstFromBatchConst(BatchSampleId, BatchSquarePos);
			float2 CircleSamplePosOffset = SampleConstFromBatchConst(BatchSampleId, BatchCircleSamplePosOffset);

			float2 SamplePosOffset[2];
			float SampleDistance;

			#if DIM_BOKEH_SIMULATION != BOKEH_SIMULATION_DISABLED && CONFIG_KERNEL_LAYOUT == KERNEL_LAYOUT_SCALABLE_DISK
			{
				const float2 LookUpUV[2] = {
					0.5 + (0.5 + CONFIG_POINT_MIRROR_BOKEH * SquarePos) / float(BOKEH_LUT_SIZE),
					0.5 + (0.5 - CONFIG_POINT_MIRROR_BOKEH * SquarePos) / float(BOKEH_LUT_SIZE),
				};
				
				SamplePosOffset[0] = KernelScope.SampleCountToSamplePosition * BokehLUT.SampleLevel(GlobalBilinearWrappedSampler, LookUpUV[0], 0).xy;

				#if DIM_BOKEH_SIMULATION == BOKEH_SIMULATION_SIMMETRIC && 0
					SamplePosOffset[1] = -SamplePosOffset[0];
				#else
					SamplePosOffset[1] = KernelScope.SampleCountToSamplePosition * BokehLUT.SampleLevel(
						GlobalBilinearWrappedSampler, LookUpUV[1], 0).xy;
				#endif
				
				SampleDistance = RingRadius;
			}
			#elif CONFIG_KERNEL_LAYOUT == KERNEL_LAYOUT_ALIGNED_SQUARE
			{
				// Do square shaped ring.
				SamplePosOffset[0] = SquarePos * KernelScope.SampleCountToSamplePosition;

				// Remove this sample at compile time if we already know there is no chance having a valid intersection.
				if (static_condition(KernelScope.bKenrelHasEnoughSamples && (length(SquarePos) > GatherParameters.MaxRingCount + 0.5)))
				{
					continue;
				}

				SamplePosOffset[1] = -SamplePosOffset[0];
				SampleDistance = length(SamplePosOffset[0] * float2(CocSqueeze, 1.0));
			}
			#elif CONFIG_KERNEL_LAYOUT == KERNEL_LAYOUT_SCALABLE_DISK
			{
				// Always keep a disk shaped kernel.
				SamplePosOffset[0] = CircleSamplePosOffset;
				SamplePosOffset[0].x *= CocInvSqueeze;

				SamplePosOffset[1] = -SamplePosOffset[0];
				SampleDistance = RingRadius;
			}
			#else
				#error Unknown gathering kernel layout.

			#endif

			
			// Fetches the mirrored samples.
			FGatherSample Sample[2];
			UNROLL
			for (uint k = 0; k < 2; k++)
			{
				const float SampleSign = (k == 0) ? 1.0 : -1.0;

				SamplePosOffset[k] = ApplyApproxPetzval(KernelScope, SamplePosOffset[k]);

				// Compute sample's input buffer UV.
				float2 SampleUV = KernelScope.KernelCenterUV + SamplePosOffset[k] * GatherInputSize.zw;

				// Sample input buffer.
				Sample[k] = FetchGatherSample(KernelScope, SampleUV);

				#if DIM_BOKEH_SIMULATION != BOKEH_SIMULATION_DISABLED && CONFIG_KERNEL_LAYOUT == KERNEL_LAYOUT_ALIGNED_SQUARE
				{
					const float LookUpRadius = 0.5 - 1.0 / float(BOKEH_LUT_SIZE);

					#if DIM_BOKEH_SIMULATION == BOKEH_SIMULATION_SIMMETRIC
						const float2 LookUpUV = LookUpRadius * CirleUnitPos;
					#elif DIM_BOKEH_SIMULATION == BOKEH_SIMULATION_GENERAL
						const float2 LookUpUV = (SampleSign * LookUpRadius) * CirleUnitPos;
					#else
						#error Unknown bokeh simulation.
					#endif

					float4 LookupSample = BokehLUT.SampleLevel(GlobalBilinearWrappedSampler, LookUpUV, 0);
					float CocRadiusToBokehEdgeDistance = LookupSample.x;

					Sample[k].CocRadius *= CocRadiusToBokehEdgeDistance;
				}
				#endif

				Sample[k].Distance = SampleDistance;
				Sample[k].Intersection = ComputeSampleIntersection(KernelScope, Sample[k].CocRadius, SampleDistance);
				
				FastAccumulator.Color += Sample[k].Color;
			}
			
			// Hand the mirrored samples to accumulator.
			AccumulateMirrorSamples(Accumulator, Sample[0], Sample[1]);
		} // for (uint BatchSampleId = 0; BatchSampleId < SamplePairBatchSize; BatchSampleId++)
	} // for (uint BatchId = 0; BatchId < RingSampleIteration; BatchId++)
}

// Gather a ring into the accumulator.
void GatherRing(
	in FGatherInputParameters GatherParameters,
	in FPrivateKernelSharedScope KernelScope,
	in uint RingId,
	in bool bIsFirstRing,
	inout FGatherAccumulator Accumulator,
	inout FFastAccumulatorOutput FastAccumulator)
{
	// Number of sample iteration for this ring.
	const uint RingSamplePairCount = GetRingSamplingPairCount(GatherParameters.KernelSamplingDensityMode, uint(RingId));

	// Distance of the ring from the center of the kernel in sample count.
	const uint RingDistance = uint(RingId + 1);
		
	FGatherRingInfos RingInfos;
	RingInfos.RingId = RingId;
	RingInfos.bIsFirstRing = bIsFirstRing;
	RingInfos.Radius = RingDistance * KernelScope.SampleCountToSamplePosition;
	RingInfos.SampleCount = RingSamplePairCount * 2;
	RingInfos.Weight = RingInfos.SampleCount * ComputeSampleWeight(RingInfos.Radius);

	#if CONFIG_SGPR_HINTS
	{
		RingInfos.Radius = ToScalarMemory(RingInfos.Radius);
	}
	#endif

	// Start accumulating ring samples
	StartRingSamples(Accumulator, RingInfos);
	
	GatherRingSamples(
		GatherParameters,
		KernelScope,
		RingId,
		bIsFirstRing,
		/* inout */ Accumulator,
		/* inout */ FastAccumulator);
	
	// End accumulating ring samples.
	EndRingSamples(Accumulator, RingInfos);
} // GatherRing()

// Gathers samples with an preconfigured accumulator and resolve its output.
uint GatherSamplesAndResolve(
	in FGatherInputParameters GatherParameters,
	in FGatherAccumulator Accumulator,
	out FAccumulatorOutput AccumulatorOutput, 
	out FFastAccumulatorOutput FastAccumulator)
{
	uint DebugGatherDensityChanges = 0;

	#if DEBUG_FORCE_LOW_RING_COUNT
		GatherParameters.MaxRingCount = 2;
	#endif
		
	FPrivateKernelSharedScope KernelScope;

	// Set up initial compile time stuf in the scope of the kernel.
	{
		KernelScope.bKenrelHasEnoughSamples = DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS;
		KernelScope.bBilinearSampleInput = (
			static_condition(GatherParameters.bFastGathering) &&
			!KernelScope.bKenrelHasEnoughSamples);
	}
	
	// Set up initial run time stuf in the scope of the kernel.
	UpdateRuntimeKernelScope(GatherParameters, /* inout */ KernelScope);

	// Initialize the fast accumulator.
	FastAccumulator.Color = 0;
	
	// Smallest ring first is only handled on KERNEL_DENSITY_CONSTANT;
	const bool bLargestRingFirst = LARGEST_RINGS_FIRST || GatherParameters.KernelSamplingDensityMode != KERNEL_DENSITY_CONSTANT;

	// Feed the center sample to the accumulator right away if smallest rings first.
	if (!bLargestRingFirst)
	{
		FGatherSample CenterSample = FetchGatherSample(KernelScope, KernelScope.KernelCenterUV);

		AccumulateCenterSampleAsItsOwnRing(Accumulator, CenterSample);
		FastAccumulator.Color += CenterSample.Color;
	}
	
	// Boolean to track the first ring at compile time.
	bool bIsFirstRing = LARGEST_RINGS_FIRST != 0;
		
	FGatherResolveInfos GatherResolveInfos;

	// Computes compile time known values about radius changes
	const float LargeKernelRadius = 1 + 2 * GatherParameters.MaxRingCount;
	const float SmallKernelRadius = 1 + 2 * (GatherParameters.MaxRingCount - GatherParameters.DensityChangeRingCount);

	const float RadiusDownScaleFactor = SmallKernelRadius / LargeKernelRadius;
					

	// TODO: reduce number of iteration as KernelRadius is getting lower than MaxCircleRingCount.
	// TODO: reduce density of at the center of kernel for fast accumulator, to reduce load on texture unit.
	if (static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_CONSTANT) ||
		static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_CONSTANT_HEXAWEB))
	{
		#if LARGEST_RINGS_FIRST
			for (int RingId = GatherParameters.MaxRingCount - 1; RingId >= 0; RingId--)
		#else
			for (int RingId = 0; RingId < int(GatherParameters.MaxRingCount); RingId++)
		#endif
		{
			GatherRing(
				GatherParameters,
				KernelScope,
				uint(RingId),
				bIsFirstRing,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);

			bIsFirstRing = false;
		} // for (int RingId)

		GatherResolveInfos.SamplesCount = GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, GatherParameters.MaxRingCount);
	}
	else if (static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_CONSTANT_DYNAMIC_RING_COUNT))
	{
		#if LARGEST_RINGS_FIRST
			for (int RingId = GatherParameters.MaxRingCount - 1; RingId >= 0; RingId--)
		#else
			for (uint RingId = 0; RingId < uint(GatherParameters.MaxRingCount); RingId++)
		#endif
		{
			GatherRing(
				GatherParameters,
				KernelScope,
				uint(RingId),
				bIsFirstRing,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);

			bIsFirstRing = false;
		} // for (int RingId)

		GatherResolveInfos.SamplesCount = GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, GatherParameters.MaxRingCount);
	}
	else if (static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_HIGHER_IN_CENTER_DISK))
	{
		//static_assert(GatherParameters.MaxRingCount >= 2, "Assumes this LARGEST_RINGS_FIRST.");
		
		int LowerRingId = GatherParameters.MaxRingCount - GatherParameters.DensityChangeRingCount;

		int RingId;

		UNROLL_N(5)
		for (RingId = GatherParameters.MaxRingCount - 1; RingId >= LowerRingId; RingId--)
		{
			GatherRing(
				GatherParameters,
				KernelScope,
				uint(RingId),
				bIsFirstRing,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);

			bIsFirstRing = false;
		} // for (int RingId)
		
		LOOP
		for (uint SamplingDensityChangeId = 0; SamplingDensityChangeId < GatherParameters.MaxDensityChangeCount; SamplingDensityChangeId++)
		{
			float MipTexelSize = pow(2.0, uint(KernelScope.GatherInputMipLevel));
			
			const float MinConvolutionRadius = GatherParameters.MaxRingCount;
			const float SmallestConvolutionRadius = GatherParameters.KernelRadius * RadiusDownScaleFactor + MipTexelSize;
	
			bool bChangeDensity = (
				GatherParameters.KernelRadius * RadiusDownScaleFactor > MinConvolutionRadius &&
				GatherParameters.MinIntersectableCocRadiusAbs < SmallestConvolutionRadius);

			BRANCH
			if (bChangeDensity)
			{
				// Decrease kernel size, and changes the already accumulated weight.
				{
					GatherParameters.KernelRadius *= RadiusDownScaleFactor;
					
					// Gives hints to the compiler that these var can be shifted to SGPR.
					#if CONFIG_SGPR_HINTS
					{
						GatherParameters.KernelRadius = ToScalarMemory(GatherParameters.KernelRadius);
					}
					#endif
					
					const float AccumulatedWeightMultiplier = rcp(RadiusDownScaleFactor * RadiusDownScaleFactor);

					AmendAccumulatedRingsWeight(/* inout */ Accumulator, GatherParameters, AccumulatedWeightMultiplier);
					AmendAccumulatedRingsWeight(/* inout */ FastAccumulator, AccumulatedWeightMultiplier);
					UpdateRuntimeKernelScope(GatherParameters, /* inout */ KernelScope);
				}

				UNROLL_N(5)
				for (int RingId2 = GatherParameters.MaxRingCount - 1; RingId2 >= LowerRingId; RingId2--)
				{
					GatherRing(
						GatherParameters,
						KernelScope,
						uint(RingId2),
						/* bIsFirstRing = */ false,
						/* inout */ Accumulator,
						/* inout */ FastAccumulator);
				} // for (int RingId)

				DebugGatherDensityChanges++;
			}
			else
			{
				break;
			}
		}

		UNROLL_N(5)
		for (; RingId >= 0; RingId--)
		{
			GatherRing(
				GatherParameters,
				KernelScope,
				uint(RingId),
				/* bIsFirstRing = */ false,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);
		} // for (int RingId)

		// TODO: this is not right, but that is fine since the only accumulator that uses that is not using this kernel density.
		GatherResolveInfos.SamplesCount = GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, GatherParameters.MaxRingCount);
	}
	else if (static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_LOWER_IN_CENTER_HEXAWEB))
	{
		int RingId = GatherParameters.MaxRingCount - 1;

		// Gather outter ring first.
		GatherRing(
			GatherParameters,
			KernelScope,
			uint(RingId),
			/* bIsFirstRing = */ true,
			/* inout */ Accumulator,
			/* inout */ FastAccumulator);

		// Changes the sampling density.
		// TODO: probably a bug in here since bokeh are not entirely great.
		{
			// Compute the number of ring remaining.
			const int RemainingRingCount = GatherParameters.MaxRingCount / 2;

			const float RadiusUpScaleFactor = float(1 + 2 * (GatherParameters.MaxRingCount - 1 )) / float(1 + 2 * RemainingRingCount);

			const float AccumulatedWeightMultiplier = rcp(RadiusUpScaleFactor * RadiusUpScaleFactor);
				
			GatherParameters.KernelRadius *= RadiusUpScaleFactor;

			AmendAccumulatedRingsWeight(/* inout */ Accumulator, GatherParameters, AccumulatedWeightMultiplier);
			AmendAccumulatedRingsWeight(/* inout */ FastAccumulator, AccumulatedWeightMultiplier);

			// TODO: looks like there is an increase in VGPR pressure in here. Drill at why.
			UpdateRuntimeKernelScope(GatherParameters, /* inout */ KernelScope);

			GatherResolveInfos.SamplesCount = (
				2 * GetRingSamplingPairCount(GatherParameters.KernelSamplingDensityMode, uint(RingId)) * AccumulatedWeightMultiplier +
				GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, RemainingRingCount));
				
			RingId = RemainingRingCount - 1;
		}

		UNROLL_N(5)
		for (; RingId >= 0; RingId--)
		{
			GatherRing(
				GatherParameters,
				KernelScope,
				uint(RingId),
				/* bIsFirstRing = */ false,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);
		} // for (int RingId)
	}
	#if CONFIG_LAYER_GATHERING && GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING
	else if (static_condition(GatherParameters.KernelSamplingDensityMode == KERNEL_DENSITY_RECURSIVE_DISKS))
	{
		//static_assert(GatherParameters.MaxRingCount >= 2, "Assumes this LARGEST_RINGS_FIRST.");
		
		{
			// Error in coc radius error introduced by the under sampling of the kernel
			const float CocRadiusError = 0.0;

			Accumulator.CloseBorderingRadius = ((GatherParameters.RingCount - GatherParameters.DensityChangeRingCount) + (0.5 + CocRadiusError)) * (Accumulator.GatherParameters.KernelRadius * rcp(0.5 + Accumulator.GatherParameters.RingCount));
			Accumulator.FarBorderingRadius = Accumulator.CloseBorderingRadius/ RadiusDownScaleFactor;

			#if CONFIG_SGPR_HINTS
			{
				Accumulator.FarBorderingRadius = ToScalarMemory(Accumulator.FarBorderingRadius);
				Accumulator.CloseBorderingRadius = ToScalarMemory(Accumulator.CloseBorderingRadius);
			}
			#endif
		}

		// Gather very first ring.
		{
			const uint RingId = GatherParameters.MaxRingCount - 1;
			GatherRing(
				GatherParameters,
				KernelScope,
				RingId,
				/* bIsFirstRing = */ true,
				/* inout */ Accumulator,
				/* inout */ FastAccumulator);
		}

		for (uint SamplingDensityChangeId = 0; SamplingDensityChangeId < GatherParameters.MaxDensityChangeCount; SamplingDensityChangeId++)
		{
			// Gather all remaining rings as single cluster of sample to the accumulator.
			{
				const uint RemainingRingCount = GatherParameters.MaxRingCount - 1;

				// Distance of the ring from the center of the kernel in sample count.
				const uint SecondRingId = GatherParameters.MaxRingCount - 2;
		
				// Number of sample iteration for this ring.
				const uint DiskSampleCount = GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, RemainingRingCount);

				// Distance of the ring from the center of the kernel in sample count.
				const uint RingDistance = uint(SecondRingId + 1);
		
				FGatherRingInfos RingInfos;
				RingInfos.RingId = SecondRingId;
				RingInfos.bIsFirstRing = false;
				RingInfos.Radius = RingDistance * KernelScope.SampleCountToSamplePosition;
				RingInfos.SampleCount = DiskSampleCount;
				RingInfos.Weight = DiskSampleCount * ComputeSampleWeight(RingInfos.Radius);
	
				// Start accumulating ring samples
				StartRingSamples(Accumulator, RingInfos);
	
				for (uint RingId = 0; RingId < RemainingRingCount; RingId++)
				{
					GatherRingSamples(
						GatherParameters,
						KernelScope,
						RingId,
						/* bIsFirstRing = */ false,
						/* inout */ Accumulator,
						/* inout */ FastAccumulator);
				}
				
				// Feed the center sample to the accumulator.
				{
					FGatherSample CenterSample = FetchGatherSample(KernelScope, KernelScope.KernelCenterUV);
					AccumulateCenterSample(Accumulator, CenterSample);
				}

				// End accumulating ring samples.
				EndRingSamples(Accumulator, RingInfos);
			}

			// Change density or stop.
			{
				float MipTexelSize = pow(2.0, uint(KernelScope.GatherInputMipLevel));
			
				const float MinConvolutionRadius = GatherParameters.MaxRingCount;
				const float SmallestConvolutionRadius = GatherParameters.KernelRadius * RadiusDownScaleFactor + MipTexelSize;
	
				bool bChangeDensity = (
					GatherParameters.KernelRadius * RadiusDownScaleFactor > MinConvolutionRadius &&
					GatherParameters.MinIntersectableCocRadiusAbs < SmallestConvolutionRadius);

				BRANCH
				if (bChangeDensity)
				{
					// Decrease kernel size.
					GatherParameters.KernelRadius *= RadiusDownScaleFactor;

					Accumulator.FarBorderingRadius = Accumulator.CloseBorderingRadius;
					Accumulator.CloseBorderingRadius = Accumulator.FarBorderingRadius * RadiusDownScaleFactor;

					// Gives hints to the compiler that these var can be shifted to SGPR.
					#if CONFIG_SGPR_HINTS
					{
						GatherParameters.KernelRadius = ToScalarMemory(GatherParameters.KernelRadius);
						Accumulator.FarBorderingRadius = ToScalarMemory(Accumulator.FarBorderingRadius);
						Accumulator.CloseBorderingRadius = ToScalarMemory(Accumulator.CloseBorderingRadius);
					}
					#endif
					
					const float AccumulatedWeightMultiplier = rcp(RadiusDownScaleFactor * RadiusDownScaleFactor);
					AmendAccumulatedRingsWeight(/* inout */ Accumulator, GatherParameters, AccumulatedWeightMultiplier);

					UpdateRuntimeKernelScope(GatherParameters, /* inout */ KernelScope);
				}
				else
				{
					break;
				}
			}

			// Gather largest ring of this smaller ring.
			{
				const uint RingId = GatherParameters.MaxRingCount - 1;
				GatherRing(
					GatherParameters,
					KernelScope,
					RingId,
					/* bIsFirstRing = */ false,
					/* inout */ Accumulator,
					/* inout */ FastAccumulator);
			}
		} // for (uint SamplingDensityChangeId = 0; SamplingDensityChangeId < GatherParameters.MaxDensityChangeCount; SamplingDensityChangeId++)

		// TODO: this is not right, but that is fine since the only accumulator that uses that is not using this kernel density.
		GatherResolveInfos.SamplesCount = GetKernelSampleCount(GatherParameters.KernelSamplingDensityMode, GatherParameters.MaxRingCount);
	}
	#endif // CONFIG_LAYER_GATHERING && GATHER_ACCUMULATOR_METHOD == GATHER_ACCUMULATOR_METHOD_RINGBINNING

	// Feed the center sample to the accumulator at the end if largest rings first.
	if (static_condition(bLargestRingFirst && GatherParameters.KernelSamplingDensityMode != KERNEL_DENSITY_RECURSIVE_DISKS))
	{
		FGatherSample CenterSample = FetchGatherSample(KernelScope, KernelScope.KernelCenterUV);

		AccumulateCenterSampleAsItsOwnRing(Accumulator, CenterSample);
		FastAccumulator.Color += CenterSample.Color;
	}

	// Resolve up fast accumulator.
	{
		// Normalise scene color to be consistent with ResolveForegroundAndBackgroundOutputs.
		FastAccumulator.Color *= rcp(GatherResolveInfos.SamplesCount);
	}
	
	// Resolves the accumulator.
	ResolveAccumulatorOutputs(Accumulator, GatherResolveInfos, /* out */ AccumulatorOutput);

	return DebugGatherDensityChanges;
}
