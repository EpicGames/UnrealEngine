// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFGatherPass.usf: gather pass entry point for diaphragm DOF.
=============================================================================*/

#include "DOFGatherCommon.ush"
#include "DOFGatherAccumulator.ush"
#include "DOFGatherKernel.ush"
#include "DOFGatherTileSuggest.ush"


//------------------------------------------------------- COMPILE TIME CONSTANTS

#define THREADGROUP_SIZEX		(COC_TILE_SIZE)
#define THREADGROUP_SIZEY		(THREADGROUP_SIZEX)
#define THREADGROUP_TOTALSIZE	(THREADGROUP_SIZEX * THREADGROUP_SIZEY)


//------------------------------------------------------- PARAMETERS

// Frame's temporal offset in pixels.
float2 TemporalJitterPixels;

// Mip bias for the gathering kernel.
float MipBias;

uint4 ViewportRect;
float2 DispatchThreadIdToInputBufferUV;
float MaxRecombineAbsCocRadius;

// Affine transformation to consider the CocRadius.
float2 ConsiderCocRadiusAffineTransformation0;
float2 ConsiderCocRadiusAffineTransformation1;
float2 ConsiderAbsCocRadiusAffineTransformation;

float2 InputBufferUVToOutputPixel;

Texture2D TileClassification_Foreground;
Texture2D TileClassification_Background;


//------------------------------------------------------- OUTPUTS

RWTexture2D<float4>	ConvolutionOutput_SceneColor;
RWTexture2D<float4>	ConvolutionOutput_SeparateAlpha;

#if CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER
RWTexture2D<float4>	ScatterOcclusionOutput;
#endif



//------------------------------------------------------- FUNCTION

/** Store the accumulator output to RWTexture.
 *
 * This is implemented as separate function to be called in different branch of GatherMainCS to help
 * the compiler reduce VGPR pressure that is critical for this amount of samples.
 */
void StoreAccumulatorOutput(in uint2 OutputPixelPosition, in FAccumulatorOutput AccumulatorOutput)
{
	if (any(OutputPixelPosition >= ViewportRect.zw))
		return;

	float4 OutScatterOcclusion = 0;
	#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY && CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER
	{
		float CocRadiusAvg = AccumulatorOutput.BackgroundCocAvgAndSquareAvg.x;

		// Max() coc variance to ensure sqrt() does not generate NaN simply on numerical errors.
		float CocRadiusVariance = max(0.0,
			AccumulatorOutput.BackgroundCocAvgAndSquareAvg.y -
			AccumulatorOutput.BackgroundCocAvgAndSquareAvg.x * AccumulatorOutput.BackgroundCocAvgAndSquareAvg.x);
		float CocRadiusStdDeviation = sqrt(CocRadiusVariance);
		
		OutScatterOcclusion.x = CocRadiusAvg;
		OutScatterOcclusion.y = max(CocRadiusVariance, 1);
	}
	#endif

	// Only output what we needs, trusting the compiler to compile out unecessary stuf.
	#if CONFIG_DOF_ALPHA
	{
		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = AccumulatorOutput.ForegroundColor * AccumulatorOutput.ForegroundAlpha;
			ConvolutionOutput_SeparateAlpha[OutputPixelPosition] = AccumulatorOutput.ForegroundAlpha;

		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
			ConvolutionOutput_SceneColor[OutputPixelPosition] = AccumulatorOutput.ForegroundHoleFillingColor;
			ConvolutionOutput_SeparateAlpha[OutputPixelPosition] = AccumulatorOutput.ForegroundHoleFillingAlpha;

		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = AccumulatorOutput.BackgroundColor;
			ConvolutionOutput_SeparateAlpha[OutputPixelPosition] = AccumulatorOutput.BackgroundWeight > 0 ? 1 : 0;
		
		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
			ConvolutionOutput_SceneColor[OutputPixelPosition] = float4(
				AccumulatorOutput.SlightFocusColor.rgb,
				AccumulatorOutput.SlightFocusOpacity);
		
		#else
			#error Unknown layer processing.

		#endif
	}
	#else // !CONFIG_DOF_ALPHA
	{
		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = float4(
				AccumulatorOutput.ForegroundColor.rgb * AccumulatorOutput.ForegroundAlpha,
				AccumulatorOutput.ForegroundAlpha);

		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
			ConvolutionOutput_SceneColor[OutputPixelPosition] = float4(
				AccumulatorOutput.ForegroundHoleFillingColor.rgb,
				AccumulatorOutput.ForegroundHoleFillingAlpha);

		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = float4(
				AccumulatorOutput.BackgroundColor.rgb,
				AccumulatorOutput.BackgroundWeight > 0 ? 1 : 0);
		
		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
			ConvolutionOutput_SceneColor[OutputPixelPosition] = float4(
				AccumulatorOutput.SlightFocusColor.rgb,
				AccumulatorOutput.SlightFocusOpacity);
		
		#else
			#error Unknown layer processing.

		#endif
	}
	#endif // !CONFIG_DOF_ALPHA

	#if CONFIG_GENERATE_SCATTER_OCCLUSION_BUFFER
		ScatterOcclusionOutput[OutputPixelPosition] = OutScatterOcclusion;

	#endif
}


//------------------------------------------------------- LDS

groupshared uint RequiresAccurateGather;


//------------------------------------------------------- ENTRY POINT

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void GatherMainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID) 
{
	ResolvedView = ResolveView();

	float4 Debug = 0;

	// Get output's UVs.
	float2 ViewportUV = (DispatchThreadId + 0.5) * ViewportSize.zw;
	float2 InputBufferCenterUV = (DispatchThreadId + 0.5) * DispatchThreadIdToInputBufferUV;

	// Sample color from larger convolution color that was gathered at lower resolution.
	FLargerConvolution LargerConvolution;
	LargerConvolution.Color = 0;
	LargerConvolution.Weight = 0;
	LargerConvolution.CocAvgAndSquareAvg = 0;
	
	// Sets the gather input compile time parameters.
	FGatherInputParameters GatherParameters;
	{
		GatherParameters.MaxRingCount = DIM_GATHER_RING_COUNT;
		GatherParameters.bFastGathering = false;

		GatherParameters.KernelSamplingDensityMode = KERNEL_DENSITY_CONSTANT;
		GatherParameters.DensityChangeRingCount = GatherParameters.MaxRingCount / 2;
		GatherParameters.MaxDensityChangeCount = CONFIG_MAX_SAMPLING_DENSITY_CHANGES;

		// TODO: variable number of ring.
		GatherParameters.RingCount = GatherParameters.MaxRingCount;
		
		GatherParameters.ConsiderCocRadiusAffineTransformation0 = kContantlyPassingAffineTransformation;
		GatherParameters.ConsiderCocRadiusAffineTransformation1 = kContantlyPassingAffineTransformation;
		GatherParameters.ConsiderAbsCocRadiusAffineTransformation = kContantlyPassingAffineTransformation;
	}

	// Setups what Coc radius are considered.
	#if 1
	{
		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
		{
			GatherParameters.ConsiderCocRadiusAffineTransformation0 = ConsiderAbsCocRadiusAffineTransformation;
		}
		#else
		{
			GatherParameters.ConsiderAbsCocRadiusAffineTransformation = ConsiderAbsCocRadiusAffineTransformation;
		}
		#endif
	}
	#else
	{
		GatherParameters.ConsiderCocRadiusAffineTransformation0 = ConsiderCocRadiusAffineTransformation0;
		GatherParameters.ConsiderCocRadiusAffineTransformation1 = ConsiderCocRadiusAffineTransformation1;
		GatherParameters.ConsiderAbsCocRadiusAffineTransformation = ConsiderAbsCocRadiusAffineTransformation;
	}
	#endif

	// Sets the per wave gather parameters
	{
		GatherParameters.DispatchThreadIdToInputBufferUV = DispatchThreadIdToInputBufferUV;
		GatherParameters.MaxRecombineAbsCocRadius = MaxRecombineAbsCocRadius;
	}
	
	// Sets the per wave line gather parameters
	{
		GatherParameters.InputBufferCenterUV = InputBufferCenterUV;
		GatherParameters.ViewportUV = ViewportUV;

		// Generate random signals
		GatherParameters.Random[0] = InterleavedGradientNoise(DispatchThreadId, 0);
		GatherParameters.Random[1] = InterleavedGradientNoise(DispatchThreadId, 1);
	}

	// Sample Coc tile texture.
	FGatheringTileSuggestion TileSuggestion;
	{
		// Actually sample the Coc tile buffer.
		#if GROUP_SIZE_IS_LARGER_THAN_COC_TILE_SIZE
			const uint2 TilePos = DispatchThreadId / COC_TILE_SIZE;
		#else
			const uint2 TilePos = GroupId;
		#endif
		const FCocTileSample CocTileInfos = LoadCocTile(COC_TILE_LAYOUT_FGD_SEP_BGD, TileClassification_Foreground, TileClassification_Background, int2(TilePos));

		TileSuggestion = InferGatherTileSuggestion(CocTileInfos, /* LayerProcessing = */ DIM_LAYER_PROCESSING);
	}
	
	// Setup first gathering's radius.
	{
		// Kernel radius is simply set to the maximum absolute Coc size.
		GatherParameters.ClosestCocRadius = TileSuggestion.ClosestCocRadius;
		GatherParameters.KernelRadius = TileSuggestion.MaxCocRadiusAbs;
		GatherParameters.MinIntersectableCocRadiusAbs = TileSuggestion.MinIntersectableCocRadiusAbs;

		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
		{
			// Dynamically control number rings.
			GatherParameters.KernelSamplingDensityMode = KERNEL_DENSITY_CONSTANT_DYNAMIC_RING_COUNT;
			GatherParameters.MaxRingCount = round(min(TileSuggestion.MaxCocRadiusAbs, SlightOutOfFocusRadiusBoundary)); // shader parameter for maximum number of rings
		}
		#endif
		
		#if CONFIG_SGPR_HINTS
		{
			GatherParameters.ClosestCocRadius = ToScalarMemory(GatherParameters.ClosestCocRadius);
			GatherParameters.KernelRadius = ToScalarMemory(GatherParameters.KernelRadius);
			GatherParameters.MinIntersectableCocRadiusAbs = ToScalarMemory(GatherParameters.MinIntersectableCocRadiusAbs);
			GatherParameters.MaxRingCount = ToScalarMemory(GatherParameters.MaxRingCount);
		}
		#endif
	}
	
	// Whether can entirely skeep gathering.
	bool bSkipGathering = TileSuggestion.bCanEarlyReturn;

	// Whether can switch to fast gathering.
	#if CONFIG_ENABLE_FAST_ACCUMULATOR && DEBUG_NO_FAST_ACCUMULATOR == 0
	bool bDoFastGathering;
	{
		#if !GROUP_SIZE_IS_LARGER_THAN_COC_TILE_SIZE
			bDoFastGathering = TileSuggestion.bCanDofastGathering;

		#elif COMPILER_SUPPORTS_WAVE_VOTE
			bDoFastGathering = WaveActiveAllTrue(TileSuggestion.bCanDofastGathering || bSkipGathering);

		#else
			RequiresAccurateGather = 0;
			GroupMemoryBarrierWithGroupSync();
			{
				uint Ignored;
				InterlockedAdd(RequiresAccurateGather, uint(!TileSuggestion.bCanDofastGathering && !bSkipGathering), Ignored);
			}
			GroupMemoryBarrierWithGroupSync();
			bDoFastGathering = (RequiresAccurateGather == 0);

		#endif
	}
	#endif

	// Early return if there is no convolution to do.
	BRANCH
	if (bSkipGathering)
	{	
		uint2 OutputPixelPosition = DispatchThreadId;

		FAccumulatorOutput AccumulatorOutput = CreateAccumulatorOutput();

		// TODO: that is not fine for LARGEST_RING_FIRST == 0
		AccumulatorOutput.BackgroundColor = LargerConvolution.Color;
		AccumulatorOutput.BackgroundWeight = LargerConvolution.Weight;
		AccumulatorOutput.BackgroundCocAvgAndSquareAvg = LargerConvolution.CocAvgAndSquareAvg;

		StoreAccumulatorOutput(OutputPixelPosition, AccumulatorOutput);

		#if DEBUG_OUTPUT
		{
			DebugOutput[OutputPixelPosition] = Debug;
		}
		#endif
	}

	// Do fast gathering because all Coc in the neighborhood are same.
	#if CONFIG_ENABLE_FAST_ACCUMULATOR && DEBUG_NO_FAST_ACCUMULATOR == 0
	else if (bDoFastGathering)
	{
		// That is sad there is not function templating, so set at compile time in the gather parameters that
		// we are actually fast gathering.
		GatherParameters.bFastGathering = true;
		GatherParameters.KernelSamplingDensityMode = CONFIG_FAST_ACCUMULATOR_KERNEL_DENSITY;
		GatherParameters.MaxRingCount = CONFIG_FAST_ACCUMULATOR_RING_COUNT;
		GatherParameters.RingCount = GatherParameters.MaxRingCount;

		// Create an accumulator to make the compiler happy, but should be entirely compile out.
		FGatherAccumulator UnusedAccumulator = CreateGatherAccumulator(GatherParameters, LargerConvolution);
		FAccumulatorOutput UnusedAccumulatorOutput = CreateAccumulatorOutput();

		// Gather the samples.
		FFastAccumulatorOutput FastAccumulatorOutput;
		GatherSamplesAndResolve(GatherParameters, UnusedAccumulator, UnusedAccumulatorOutput, FastAccumulatorOutput);

		// Ignore Accumulator, just return the fast accumulator.
		FAccumulatorOutput AccumulatorOutput = CreateAccumulatorOutput();
		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
		{
			AccumulatorOutput.ForegroundColor = FastAccumulatorOutput.Color;
			AccumulatorOutput.ForegroundAlpha = saturate(GatherParameters.KernelRadius - (GatherParameters.MaxRecombineAbsCocRadius - 1.0));
		}
		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
		{
			AccumulatorOutput.BackgroundColor = FastAccumulatorOutput.Color;
			AccumulatorOutput.BackgroundWeight = 1;

			AccumulatorOutput.BackgroundCocAvgAndSquareAvg.x = GatherParameters.KernelRadius;
			AccumulatorOutput.BackgroundCocAvgAndSquareAvg.y = 0;
		}
		#else
			#error Fast gathering is not supported.
		#endif
		
		#if DEBUG_FAST_ACCUMULATOR
			AccumulatorOutput.ForegroundColor.rgb *= float3(0.5, 1, 0.5);
			AccumulatorOutput.BackgroundColor.rgb *= float3(0.5, 1, 0.5);
		#endif
		
		#if 1 // Lower VGPR footprint.
			uint2 OutputPixelPosition = uint2(GatherParameters.InputBufferCenterUV * InputBufferUVToOutputPixel);
		#else
			uint2 OutputPixelPosition = DispatchThreadId;
		#endif

		Debug = float4(0, 1, 0, 0);

		StoreAccumulatorOutput(OutputPixelPosition, AccumulatorOutput);

		#if DEBUG_OUTPUT
		{
			DebugOutput[OutputPixelPosition] = Debug;
		}
		#endif
	}
	#endif // ALLOW_FAST_ACCUMULATOR

	// Do gathering with FGatherAccumulator.
	else
	{
		// Drives kernel sampling density.
		#if CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER == 0
			// NOP
		#elif CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER == 1
		{
			#if DIM_LAYER_PROCESSING != LAYER_PROCESSING_BACKGROUND_ONLY
				#error Kernel sampling density driver only available for background.
			#endif

			#if CONFIG_LAYER_GATHERING
				GatherParameters.KernelSamplingDensityMode = KERNEL_DENSITY_RECURSIVE_DISKS;
			#else
				GatherParameters.KernelSamplingDensityMode = KERNEL_DENSITY_HIGHER_IN_CENTER_DISK;
			#endif
		}
		#endif

		// Create, use and resolve gather accumulator.
		FAccumulatorOutput AccumulatorOutput = CreateAccumulatorOutput();
		uint DebugGatherDensityChanges;
		{
			FFastAccumulatorOutput UnusedFastAccumulatorOutput;

			FGatherAccumulator FirstAccumulator = CreateGatherAccumulator(GatherParameters, LargerConvolution);
			DebugGatherDensityChanges = GatherSamplesAndResolve(GatherParameters, FirstAccumulator, AccumulatorOutput, UnusedFastAccumulatorOutput);
		}

		#if DEBUG_FAST_ACCUMULATOR
		{
			#if CONFIG_KERNEL_SAMPLING_DENSITY_DRIVER > 0
			UNROLL_N(CONFIG_MAX_SAMPLING_DENSITY_CHANGES)
			for (uint i = 0; i < CONFIG_MAX_SAMPLING_DENSITY_CHANGES; i++)
			{
				if (i < DebugGatherDensityChanges)
				{
					AccumulatorOutput.ForegroundColor.rgb *= float3(1, 0.5, 0.5);
					AccumulatorOutput.BackgroundColor.rgb *= float3(1, 0.5, 0.5);
				}
			}
			#endif

			AccumulatorOutput.ForegroundColor.rgb *= float3(1, 0.5, 0.5);
			AccumulatorOutput.BackgroundColor.rgb *= float3(1, 0.5, 0.5);
		}
		#endif
		
		#if 1 // Lower VGPR footprint.
			uint2 OutputPixelPosition = uint2(GatherParameters.InputBufferCenterUV * InputBufferUVToOutputPixel);
		#else
			uint2 OutputPixelPosition = DispatchThreadId;
		#endif

		#if 1
		{
			#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
			{
			}
			#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
			{
			}
			#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			{
			}
			#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS
			{
				Debug = float4(GatherParameters.MaxRingCount, 0, 0, 0);
			}
			#else
				#error Unknown layer processing.

			#endif
		}
		#elif 1
		{
			Debug = float4(GatherParameters.KernelRadius, 0, 0, 0);
		}
		#elif 0
		{
			Debug = float4(1, 0, 0, 0);
		}
		#endif

		StoreAccumulatorOutput(OutputPixelPosition, AccumulatorOutput);

		#if DEBUG_OUTPUT
		{
			DebugOutput[OutputPixelPosition] = Debug;
		}
		#endif
	}
}
