// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFGatherTileSuggest.ush: takes care of choosing how to
	gather a given coc tile.
=============================================================================*/

#pragma once

#include "DOFCocTileCommon.ush"


//------------------------------------------------------- SHADER_PARAMETER

float MinGatherRadius;
float SlightOutOfFocusRadiusBoundary;


//------------------------------------------------------- FUNCTIONS

struct FGatheringTileSuggestion
{
	// True when we know from coc tile that there is no gathering.
	bool bCanEarlyReturn;

	// True when we can use fast gather accumulator.
	bool bCanDofastGathering;
	
	// Smallest absolute coc radius
	float MinCocRadiusAbs;
	
	// Smallest absolute coc radius
	float MinIntersectableCocRadiusAbs;
	
	// Largest absolute coc radius
	float MaxCocRadiusAbs;

	// Coc radius the closest from the camera.
	float ClosestCocRadius;
};


/** Infers the gather processing to do from coc tile buffer sample, for a given LayerProcessing. */
FGatheringTileSuggestion InferGatherTileSuggestion(
	in FCocTileSample CocTileInfos,
	const uint LayerProcessing)
{
	FGatheringTileSuggestion OutSuggestion;

	bool bIsComplexTile;
	
	// May not have this information, and it may not be needed.
	OutSuggestion.MinIntersectableCocRadiusAbs = EXTREMELY_LARGE_COC_RADIUS;

	if (LayerProcessing == LAYER_PROCESSING_FOREGROUND_ONLY)
	{
		// Smallest Coc radius is the foreground minimum Coc.
		OutSuggestion.MinCocRadiusAbs = -CocTileInfos.FgdMaxCoc;
		
		// Kernel radius is the foreground maximum Coc.
		OutSuggestion.MaxCocRadiusAbs = -CocTileInfos.FgdMinCoc;
		
		// Closest Coc radius is the foreground minimum Coc.
		OutSuggestion.ClosestCocRadius = CocTileInfos.FgdMinCoc;
	}
	else if (LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		// Smallest Coc radius is the foreground minimum Coc.
		OutSuggestion.MinCocRadiusAbs = -CocTileInfos.FgdMaxCoc;

		// Kernel radius is the foreground maximum Coc.
		// TODO: max() with background.
		OutSuggestion.MaxCocRadiusAbs = -CocTileInfos.FgdMinCoc;
			
		// Closest Coc radius is the foreground minimum Coc.
		OutSuggestion.ClosestCocRadius = CocTileInfos.FgdMinCoc;
	}
	else if (LayerProcessing == LAYER_PROCESSING_BACKGROUND_ONLY)
	{
		// Smallest Coc radius is the backrgound minimum Coc.
		OutSuggestion.MinCocRadiusAbs = CocTileInfos.BgdMinCoc;
		
		// Don't have this information, so just assume this is min absolute coc radius
		OutSuggestion.MinIntersectableCocRadiusAbs = CocTileInfos.BgdMinIntersectableCoc;

		// Kernel radius is the foreground maximum Coc.
		OutSuggestion.MaxCocRadiusAbs = CocTileInfos.BgdMaxCoc;
		
		// Closest Coc radius is the background minimum Coc.
		OutSuggestion.ClosestCocRadius = CocTileInfos.BgdMinCoc;
	}
	else // if (LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		// TODO: could be optimised on transition between very close foreground and very far background to bCanSkipGathering == true.
		if (CocTileInfos.FgdMinCoc == 0 && CocTileInfos.BgdMaxCoc == 0)
		{
			OutSuggestion.MinCocRadiusAbs = EXTREMELY_LARGE_COC_RADIUS;
		}
		else if (CocTileInfos.FgdMinCoc == 0)
		{
			OutSuggestion.MinCocRadiusAbs = CocTileInfos.BgdMinCoc;
		}
		else if (CocTileInfos.BgdMaxCoc == 0)
		{
			OutSuggestion.MinCocRadiusAbs = -CocTileInfos.FgdMaxCoc;
		}
		else
		{
			OutSuggestion.MinCocRadiusAbs = min(-CocTileInfos.FgdMaxCoc, CocTileInfos.BgdMinCoc);
		}
		
		OutSuggestion.MaxCocRadiusAbs = max(-CocTileInfos.FgdMinCoc, CocTileInfos.BgdMaxCoc);

		OutSuggestion.ClosestCocRadius = CocTileInfos.FgdMinCoc < 0 ? CocTileInfos.FgdMinCoc : CocTileInfos.BgdMinCoc;
	}
	
	// Coc error accepted for fast gathering, set to 5% of the kernel size.
	float FastGatheringCocError = OutSuggestion.MaxCocRadiusAbs * FAST_GATHERING_COC_ERROR;

	// Logic whether can entirely skeep gathering.
	if (LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		// Foreground hole filling only need to accumulate where we know foreground's opacity != 0 && != 1.
		OutSuggestion.bCanEarlyReturn = (
			OutSuggestion.MaxCocRadiusAbs - OutSuggestion.MinCocRadiusAbs < FastGatheringCocError ||
			OutSuggestion.MinCocRadiusAbs > MAX_RECOMBINE_ABS_COC_RADIUS ||
			(OutSuggestion.MaxCocRadiusAbs <= MinGatherRadius));
	}
	else if (LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		OutSuggestion.bCanEarlyReturn = (OutSuggestion.MinCocRadiusAbs > SlightOutOfFocusRadiusBoundary);
	}
	else
	{
		OutSuggestion.bCanEarlyReturn = (OutSuggestion.MaxCocRadiusAbs <= MinGatherRadius);
	}
	
	// Whether can switch to fast gathering.
	OutSuggestion.bCanDofastGathering = OutSuggestion.MaxCocRadiusAbs - OutSuggestion.MinCocRadiusAbs < FastGatheringCocError;

	return OutSuggestion;
}
