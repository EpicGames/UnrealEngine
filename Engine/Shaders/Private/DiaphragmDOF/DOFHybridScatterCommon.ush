// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFHybridScatterCommon.ush: Hold code shared between
	hybrid scattering's compilation and scattering passes.
=============================================================================*/

#include "DOFCommon.ush"


//------------------------------------------------------- DEBUG COMPILE TIME CONFIG

// Mode to debug hybrid scatering mode.
//  0: Disabled,
//  1: Shows scattered sprites that actually draw no color in red, other in green.
//  2: Shows scattered boked in red.
#define DEBUG_HYBRID_SCATTERING_SPRITE 0


//------------------------------------------------------- ENUM VALUES

/** Indirect bokeh drawing method. */
	// Bokeh is drawn using DrawPrimitiveIndirect() with triangle list topology.
	#define BOKEH_DRAW_METHOD_TRIANGLE 0
	
	// Bokeh is drawn using DrawIndexedPrimitiveIndirect() with triangle list topology.
	#define BOKEH_DRAW_METHOD_INDEXED_TRIANGLE 1
	
	// Bokeh is drawn using DrawPrimitiveIndirect() with rect list topology.
	#define BOKEH_DRAW_METHOD_RECT 2


//------------------------------------------------------- COMPILE TIME CONSTANTS

// Hybrid scattering is done by square shaped group of sample to reduce overlapping
// pixel shader invocation. This constant is the size in sample of this group.
#define HYBRID_SCATTERING_GROUP_BORDER_SIZE 2


// Number of pixels per scattering group.
#define PIXEL_COUNT_PER_SCATTER_GROUP (HYBRID_SCATTERING_GROUP_BORDER_SIZE * HYBRID_SCATTERING_GROUP_BORDER_SIZE)

// 2x float4: 2 floats + (3 flags * 2 floats)
#define VIGNETTE_DATA_PER_PIXEL (2)

#define SCATTER_DRAW_LIST_DATA_PER_PIXEL (1 + VIGNETTE_DATA_PER_PIXEL)

// Number of float4s in the draw list per scattering group
#define SCATTER_DRAW_LIST_GROUP_STRIDE (1 + (PIXEL_COUNT_PER_SCATTER_GROUP * SCATTER_DRAW_LIST_DATA_PER_PIXEL))


// Configures the drawing method used for scattering bokeh.
#if PLATFORM_SUPPORTS_RECT_LIST
	#define BOKEH_DRAW_METHOD (BOKEH_DRAW_METHOD_RECT)
#else
	#define BOKEH_DRAW_METHOD (BOKEH_DRAW_METHOD_INDEXED_TRIANGLE)
#endif


// The intersection model of the scater pass do a 1 pixel thick Feather for anti aliasing, where half
// intensity is excatly set a CocRadius distance for energy conservativness.
#define ANTI_ALIASING_FEATHER_OFFSET 0.5

// Whether to pack multiple sprite in same indirect draw instance.
#define SCATTERING_GROUP_PACKING 1

// Turns on/off the taylor expensition optimisation for bokeh distance
#define TAYLOR_EXPENDED_COC_INTERSECTION 1

// Number vertex shader invocation that should be targeted.
#define VERTEX_SHADER_INVOCATION_TARGET 64


//------------------------------------------------------- COMPILE TIME DERIVED CONSTANTS

// Setup the number drawing related constant according to the method use.
#if BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_TRIANGLE
	#define VERTEX_PER_SCATTERING_GROUP 6
	#define VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP 6
	#define SCATTERING_INDIRECT_PARAMETER_SIZE (DRAW_INDIRECT_UINT_COUNT)

#elif BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_INDEXED_TRIANGLE
	#define VERTEX_PER_SCATTERING_GROUP 6
	#define VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP 4
	#define SCATTERING_INDIRECT_PARAMETER_SIZE (DRAW_INDEXED_INDIRECT_UINT_COUNT)

#elif BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_RECT
	#define VERTEX_PER_SCATTERING_GROUP 3
	#define VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP 3
	#define SCATTERING_INDIRECT_PARAMETER_SIZE (DRAW_INDIRECT_UINT_COUNT)

#else
	#error Unknown bokeh drawing method.

#endif

#if DIM_SCATTER_QUALITY == SCATTER_QUALITY_MEDIUM
	#define CONFIG_PETZVAL 0
	#define CONFIG_VIGNETTE 0
#elif DIM_SCATTER_QUALITY == SCATTER_QUALITY_HIGH
	#define CONFIG_PETZVAL 1
	#define CONFIG_VIGNETTE 1
#endif //DIM_SCATTER_QUALITY