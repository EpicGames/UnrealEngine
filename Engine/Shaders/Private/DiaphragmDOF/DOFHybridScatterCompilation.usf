// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFHybridScatterCompilation.usf: Compile list of sample to scatter.
=============================================================================*/

#include "DOFHybridScatterCommon.ush"


//------------------------------------------------------- COMPILE TIME CONSTANTS

#define SCATTER_GROUP_PER_INSTANCE (VERTEX_SHADER_INVOCATION_TARGET / VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP)


//------------------------------------------------------- PARAMETERS

uint MaxScatteringGroupCount;


//------------------------------------------------------- OUTPUTS

RWBuffer<uint> OutScatterDrawIndirectParameters;

RWStructuredBuffer<float4> OutForegroundScatterDrawList;
RWStructuredBuffer<float4> OutBackgroundScatterDrawList;


//------------------------------------------------------- LDS

groupshared uint ScatterGroupCount;


//------------------------------------------------------- ENTRY POINT

#if SCATTERING_GROUP_PACKING
[numthreads(1, SCATTER_GROUP_PER_INSTANCE, 1)]
#else
[numthreads(1, 1, 1)]
#endif
void ScatterGroupPackMainCS(
	uint GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex) 
{
	// Prepares the draw indirect parameters.
	if (GroupThreadIndex < 1)
	{
		uint UAVOffset = SCATTERING_INDIRECT_PARAMETER_SIZE * GroupId;
	
		// Min because might have atomically incremented more in reduce pass.
		ScatterGroupCount = min(OutScatterDrawIndirectParameters[UAVOffset + 1], MaxScatteringGroupCount);

		#if SCATTERING_GROUP_PACKING
		{
			uint InstanceCount = (ScatterGroupCount + (SCATTER_GROUP_PER_INSTANCE - 1)) / SCATTER_GROUP_PER_INSTANCE;

			// Outputs FRHIDrawIndirectParameters's VertexCountPerInstance or FRHIDrawIndexedIndirectParameters's IndexCountPerInstance.
			OutScatterDrawIndirectParameters[UAVOffset + 0] = VERTEX_PER_SCATTERING_GROUP * SCATTER_GROUP_PER_INSTANCE;

			// Outputs FRHIDrawIndirectParameters & FRHIDrawIndexedIndirectParameters's InstanceCount.
			OutScatterDrawIndirectParameters[UAVOffset + 1] = InstanceCount;
		}
		#else
		{
			// Outputs FRHIDrawIndirectParameters's VertexCountPerInstance or FRHIDrawIndexedIndirectParameters's IndexCountPerInstance.
			OutScatterDrawIndirectParameters[UAVOffset + 0] = VERTEX_PER_SCATTERING_GROUP;
			
			// Outputs FRHIDrawIndirectParameters & FRHIDrawIndexedIndirectParameters's InstanceCount.
			OutScatterDrawIndirectParameters[UAVOffset + 1] = ScatterGroupCount;
		}
		#endif
	}
	
	#if SCATTERING_GROUP_PACKING
	{
		// Group sync for ScatterGroupCount to be available.
		GroupMemoryBarrierWithGroupSync();

		// Clears last VERTEX_PER_SCATTERING_GROUP so we don't end up drawing last scattering group id >= ScatterGroupCount
		// when ScatterGroupCount is not a multiple of SCATTER_GROUP_PER_INSTANCE
		BRANCH
		if (GroupId == 0)
		{
			#if DIM_HYBRID_SCATTER_FGD
			for (uint i = 0; i < SCATTER_DRAW_LIST_GROUP_STRIDE; i++)
			{
				OutForegroundScatterDrawList[ScatterGroupCount * SCATTER_DRAW_LIST_GROUP_STRIDE + GroupThreadIndex + i * SCATTER_GROUP_PER_INSTANCE] = float4(0, 0, 0, 0);
			}
			#endif
		}
		else
		{	
			#if DIM_HYBRID_SCATTER_BGD
			for (uint i = 0; i < SCATTER_DRAW_LIST_GROUP_STRIDE; i++)
			{
				OutBackgroundScatterDrawList[ScatterGroupCount * SCATTER_DRAW_LIST_GROUP_STRIDE + GroupThreadIndex + i * SCATTER_GROUP_PER_INSTANCE] = float4(0, 0, 0, 0);
			}
			#endif
		}
	}
	#endif
}
