// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFHybridScatterCompilation.ush: Compile list of sample to scatter.
=============================================================================*/

#include "DOFHybridScatterCommon.ush"
#include "../EyeAdaptationCommon.ush"


//------------------------------------------------------- PARAMETERS

float NeighborCompareMaxColor;

// Parameters for ComputeScatterFactor.
struct FHybridScatterInputs
{
	// ------- compile time constant category.

	// Texture used for performing neighborhood comparison.
	Texture2D NeighborhoodComparisonBuffer;
	float2 NeighborhoodComparisonBufferInvExtent;
	float4 NeighborhoodComparisonViewRectUV;
	

	// ------- run time pixel independent category.
	float FrameExposureScale;
	
	// ------- run time category.
	float4 Color;
	float CocRadius;

	// Coordinate of input pixel.
	float2 BufferUV;
};

// Return the minimal luma for a pixel to be scattered.
float ComputeLuminanceOnlyBasedScatterFactor(in const FHybridScatterInputs Parameters)
{
	const float MinimalPerceivedLuma4 = 0.4;

	float PerceivedLuma4 = Parameters.FrameExposureScale * Luma4(Parameters.Color.rgb);

	return saturate(PerceivedLuma4 * rcp(MinimalPerceivedLuma4) - 1.0);
	//return saturate((PerceivedLuminance - MinimalPerceivedLuma4) * rcp(MinimalPerceivedLuma4));
}

// Determines whether a sample should be scattered.
float ComputeNeighborBasedScatterFactor(in const FHybridScatterInputs Parameters)
{
	// Neighborhood comparison to decide whether should scatter or not.
	float ScatterFactor = 0;
	{
		// Minimal threshold.
		const float Threshold = 0.7;

		float LinearScale = rcp(Parameters.FrameExposureScale);
		float LinearColorMaximum = NeighborCompareMaxColor * LinearScale;

		float4 SaturatedColor = min(Parameters.Color, LinearColorMaximum);

		UNROLL
		for (uint i = 0; i < 4; i++)
		{
			const float2 Offset = kOffsetsCross3x3[i];
			if (all(Offset == 0))
			{
				continue;
			}

			float2 SampleUV = Parameters.BufferUV + Offset * Parameters.NeighborhoodComparisonBufferInvExtent;
			SampleUV = clamp(SampleUV, Parameters.NeighborhoodComparisonViewRectUV.xy, Parameters.NeighborhoodComparisonViewRectUV.zw);

			float4 SampleColor = Parameters.NeighborhoodComparisonBuffer.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);
			float SampleCoc = SampleColor.a;

			SampleColor = min(SampleColor, LinearColorMaximum);


			float4 ColorDiff = max(SaturatedColor - SampleColor, 0);

			// TODO: should do Luminance() instead?
			float MajorDiff = max3(ColorDiff.r, ColorDiff.g, ColorDiff.b) * (Parameters.FrameExposureScale);

			float SampleScatterFactor = saturate(MajorDiff * rcp(Threshold) - 1.0);

			// Wether the sample should be considered because have CoC with same sign as input Coc.
			float ConsiderSample = 1; // saturate(SampleCoc * Parameters.CocRadius * 1000000.0);

			if (i == 0)
			{
				ScatterFactor = SampleScatterFactor * ConsiderSample;
			}
			else
			{
				ScatterFactor = max(ScatterFactor, SampleScatterFactor * ConsiderSample);
			}
		}
	}

	return ScatterFactor;
}
