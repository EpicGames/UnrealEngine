// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFHybridScatterPixelShader.usf: Scattering's pixel shader.
=============================================================================*/

#include "DOFHybridScatterCommon.ush"
#include "DOFVignetteCommon.ush"
#include "../FastMath.ush"
#include "/Engine/Public/DualPixelVectorization.ush"

//------------------------------------------------------- COMPILE TIME CONFIG

// Whether to point mirror the bokeh.
#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
	#define CONFIG_POINT_MIRROR_BOKEH -1
#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
	#define CONFIG_POINT_MIRROR_BOKEH 1
#endif


//------------------------------------------------------- PARAMETERS

float ScatteringScaling;

StructuredBuffer<float4> ScatterDrawList;

Texture2D BokehLUT;

float4 ScatterOcclusionSize;
Texture2D ScatterOcclusion;

float BarrelRadius;
float BarrelLength;

//------------------------------------------------------- FUNCTIONS


half2 HalfSafeRcp(half2 Value)
{
	return half2(
		Value.x > 0.0 ? rcp(Value.x) : 0.0,
		Value.y > 0.0 ? rcp(Value.y) : 0.0
	);
}

half2x2 dpv_mul_vec_mat(half2x2 InVector, half2x2 InMatrix)
{
	return dpv_interleave_registers(
		mul(dpv_lo(InVector), InMatrix), 
		mul(dpv_hi(InVector), InMatrix)
	);
}

/// Calculate the visibility of the pixel for a specific bokeh through the lens barrel
half2 GetPixelVisibility_DPV(half2x2 PixelOffset, uint2 VignetteDataOffset)
{
#if CONFIG_VIGNETTE
	bool bOpticalVignetting = BarrelLength >= 0;
	if(bOpticalVignetting)
	{
		float4 VignetteData[VIGNETTE_DATA_PER_PIXEL][DPV_PIXEL_PER_LANE] = {
			{
				ScatterDrawList[VignetteDataOffset[0] + 0],
				ScatterDrawList[VignetteDataOffset[1] + 0]
			},
			{
				ScatterDrawList[VignetteDataOffset[0] + 1],
				ScatterDrawList[VignetteDataOffset[1] + 1]
			}
		};

		uint2 BarrelData[DPV_PIXEL_PER_LANE] = {
			asuint(VignetteData[0][0].xy),
			asuint(VignetteData[0][1].xy)
		};

		uint2 ProjectedMatteBoxFlags[MAX_MATTE_BOX_FLAGS][DPV_PIXEL_PER_LANE] = {
			{
				asuint(VignetteData[0][0].zw),
				asuint(VignetteData[0][1].zw)
			},
			{
				asuint(VignetteData[1][0].xy),
				asuint(VignetteData[1][1].xy)
			},
			{
				asuint(VignetteData[1][0].zw),
				asuint(VignetteData[1][1].zw)
			}
		};

		half2x2 SamplePositionAtBarrelEnd;
		half2 Visibility = HitTestProjectedCylinder_DPV(PixelOffset, BarrelData, BarrelRadius, /* out */ SamplePositionAtBarrelEnd);

		UNROLL
		for (int Index = 0; Index < MAX_MATTE_BOX_FLAGS; ++Index)
		{
			Visibility *= HitTestProjectedMatteBoxFlag_DPV(SamplePositionAtBarrelEnd, ProjectedMatteBoxFlags[Index]);
		}

		return Visibility;
	}
#endif
	return half2(1.0, 1.0);
}

void ScatterMainPS(
	in nointerpolation float4 BokehCenterScreenAndClipPos : TEXCOORD0,
	in nointerpolation float4 ColorAndAbsCocRadius0 : TEXCOORD1,
	in nointerpolation float4 ColorAndAbsCocRadius1 : TEXCOORD2,
	in nointerpolation uint3 GlobalSpriteIdAndPetzvalTransformPacked : TEXCOORD3,
#if TAYLOR_EXPENDED_COC_INTERSECTION
	in nointerpolation float4 TaylorExpensionMAD0   : TEXCOORD4,
	in nointerpolation float4 TaylorExpensionMAD1   : TEXCOORD5,
#endif
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0) 
{
	ResolvedView = ResolveView();

	OutColor = 0;

	const float2 BokehCenter0 = BokehCenterScreenAndClipPos.xy;
	const float2 BokehCenterClipPos = BokehCenterScreenAndClipPos.zw;

	half3 Color[PIXEL_COUNT_PER_SCATTER_GROUP];
	half AbsCocRadius[PIXEL_COUNT_PER_SCATTER_GROUP];

	{
		half2 Unpacked0;
		half2 Unpacked1;

		Unpacked0 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius0.x));
		Unpacked1 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius0.y));
		Color[0] = half3(Unpacked0.xy, Unpacked1.x);
		AbsCocRadius[0] = Unpacked1.y;

		Unpacked0 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius0.z));
		Unpacked1 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius0.w));
		Color[1] = half3(Unpacked0.xy, Unpacked1.x);
		AbsCocRadius[1] = Unpacked1.y;

		Unpacked0 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius1.x));
		Unpacked1 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius1.y));
		Color[2] = half3(Unpacked0.xy, Unpacked1.x);
		AbsCocRadius[2] = Unpacked1.y;

		Unpacked0 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius1.z));
		Unpacked1 = (half2)UnpackFloat2FromUInt(asuint(ColorAndAbsCocRadius1.w));
		Color[3] = half3(Unpacked0.xy, Unpacked1.x);
		AbsCocRadius[3] = Unpacked1.y;
	}
	
	#if TAYLOR_EXPENDED_COC_INTERSECTION
		half2 TaylorExpension[4] = {
			half2(TaylorExpensionMAD0.xy),
			half2(TaylorExpensionMAD0.zw),
			half2(TaylorExpensionMAD1.xy),
			half2(TaylorExpensionMAD1.zw),
		};
	#endif

	#if DIM_SCATTER_OCCLUSION
		float4 ScatterOcclusionSample = ScatterOcclusion.SampleLevel(GlobalBilinearWrappedSampler, SvPosition.xy * ScatterOcclusionSize.zw, 0);

		half EncodedCocRadiusAvg = half(ScatterOcclusionSample.x);
		half EncodedCocRadiusVariance = half(ScatterOcclusionSample.y);
	#endif
	
	half2 PixelOffset0 = half2(SvPosition.xy - BokehCenter0);

	uint GlobalSpriteId = GlobalSpriteIdAndPetzvalTransformPacked.x;
#if CONFIG_PETZVAL
	half2x2 PetzvalTransform = half2x2(
		UnpackFloat2FromUInt(GlobalSpriteIdAndPetzvalTransformPacked.y),
		UnpackFloat2FromUInt(GlobalSpriteIdAndPetzvalTransformPacked.z)
	);
#endif

	// Evaluate all bokeh of the scattering group.
	// Dual Pixel Vectorization: evaluate 2 pixels at a time using packed 16-bit float instructions on supported platforms.
	UNROLL
	for (uint i = 0; i < PIXEL_COUNT_PER_SCATTER_GROUP; i += DPV_PIXEL_PER_LANE)
	{
		// Pixel coordinate of the center of the bokeh.
		half2x2 PixelLocalOffset = dpv_interleave_registers(half2(kSquare2x2[i]), half2(kSquare2x2[i + 1]));
		half2x2 PixelOffset = dpv_sub(PixelOffset0, dpv_scale(half(ScatteringScaling), PixelLocalOffset));

		half2 PixelAbsCocRadius = dpv_interleave_registers(AbsCocRadius[i], AbsCocRadius[i+1]);

#if CONFIG_PETZVAL		
		PixelOffset = dpv_mul_vec_mat(PixelOffset, PetzvalTransform);
#endif

		PixelOffset[0] *= half(CocSqueeze);

		half2 Intersection;
		#if DIM_BOKEH_SIMULATION
		{
			half2 PixelDistance = dpv_length(PixelOffset);
			half2x2 LookUpUV = dpv_scale(PixelOffset, half(CONFIG_POINT_MIRROR_BOKEH * (0.5 - 1.0 / float(BOKEH_LUT_SIZE))) * HalfSafeRcp(PixelDistance));

			// TODO: might be faster without using view uniform buffer.
			float4 LookupSample0 = BokehLUT.SampleLevel(GlobalBilinearWrappedSampler, dpv_lo(LookUpUV), 0);
			float4 LookupSample1 = BokehLUT.SampleLevel(GlobalBilinearWrappedSampler, dpv_hi(LookUpUV), 0);
			half2 CocRadiusToBokehEdgeFactor = dpv_interleave_registers(half(LookupSample0.x), half(LookupSample1.x));

			Intersection = saturate(PixelAbsCocRadius * CocRadiusToBokehEdgeFactor - PixelDistance + ANTI_ALIASING_FEATHER_OFFSET);
		}
		#elif TAYLOR_EXPENDED_COC_INTERSECTION
		{
			half2 TaylorExpansionX = dpv_interleave_registers(TaylorExpension[i].x, TaylorExpension[i+1].x);
			half2 TaylorExpansionY = dpv_interleave_registers(TaylorExpension[i].y, TaylorExpension[i+1].y);
			Intersection = saturate(dpv_dot(PixelOffset, PixelOffset) * TaylorExpansionX + TaylorExpansionY);
		}
		#else
		{
			// Pretty simple sphere intersection.
			half2 PixelDistance = dpv_length(PixelOffset);
			Intersection = saturate(PixelAbsCocRadius - PixelDistance + ANTI_ALIASING_FEATHER_OFFSET);
		}
		#endif

		BRANCH
		if (any(Intersection > 0))
		{
			half2 Visibility = 1;
			#if DIM_SCATTER_OCCLUSION
			{
				half2 EncodedPixelAbsCocRadius = PixelAbsCocRadius / ScatteringScaling;
				half2 CocRadiusComparison = max(EncodedPixelAbsCocRadius * half(1 - FAST_GATHERING_COC_ERROR) - EncodedCocRadiusAvg, (half2)0);
				Visibility = EncodedCocRadiusVariance / (EncodedCocRadiusVariance + CocRadiusComparison * CocRadiusComparison);
			}
			#endif

			uint2 ScatterDrawListOffset = {
				SCATTER_DRAW_LIST_GROUP_STRIDE * GlobalSpriteId + 1 + ((i + 0) * SCATTER_DRAW_LIST_DATA_PER_PIXEL),
				SCATTER_DRAW_LIST_GROUP_STRIDE * GlobalSpriteId + 1 + ((i + 1) * SCATTER_DRAW_LIST_DATA_PER_PIXEL)
			};
			uint2 VignetteDataOffset = ScatterDrawListOffset + 1;
		
			Visibility *= GetPixelVisibility_DPV(PixelOffset, VignetteDataOffset);

			half3x2 PixelColor = dpv_interleave_registers(Color[i].rgb, Color[i + 1].rgb);
			PixelColor = dpv_scale(PixelColor, Intersection * Visibility);
			half3 ColorAcc = dpv_lo(PixelColor) + dpv_hi(PixelColor);
			OutColor.rgb += ColorAcc;
		}
	}
	
	#if DEBUG_HYBRID_SCATTERING_SPRITE == 1
		if (any(OutColor.rgb > 0))
			OutColor = float4(0, 0.1, 0, 0);
		else
			OutColor = float4(0.1, 0, 0, 0);

	#elif DEBUG_HYBRID_SCATTERING_SPRITE == 2
		OutColor *= float4(1, 0, 0, 0);

	#endif
}
