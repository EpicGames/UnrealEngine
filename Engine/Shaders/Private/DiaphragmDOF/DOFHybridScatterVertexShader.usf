// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFHybridScatterVertexShader.usf: Scattering's vertex shader.
=============================================================================*/

#include "DOFHybridScatterCommon.ush"


//------------------------------------------------------- COMPILE TIME CONSTANTS

static const float2 kSpriteVertices[6] = {
	float2(-1.0, +1.0),
	float2(+1.0, +1.0),
	float2(+1.0, -1.0),
	float2(-1.0, -1.0),
	float2(+1.0, -1.0),
	float2(-1.0, +1.0),
};


//------------------------------------------------------- PARAMETERS

float4 ViewportSize;

float ScatteringScaling;
float CocRadiusToCircumscribedRadius;
StructuredBuffer<float4> ScatterDrawList;

float Petzval;
float PetzvalFalloffPower;
float2 PetzvalExclusionBoxExtents;
float PetzvalExclusionBoxRadius;

//------------------------------------------------------- FUNCTIONS

float2x2 CalcPetzvalTransform(float2 BokehCenter)
{
	float2x2 Transform = {1,0,0,1};
#if CONFIG_PETZVAL
	if (Petzval != 0)
	{
		const float Radius = PetzvalExclusionBoxRadius * min(PetzvalExclusionBoxExtents.x, PetzvalExclusionBoxExtents.y);
		const float2 BoxToBokeh = sign(BokehCenter) * max(abs(BokehCenter) - PetzvalExclusionBoxExtents + Radius, 0.0);
		float BokehDistance = max(0.0, length(BoxToBokeh));
		const float2 Normal = BoxToBokeh / BokehDistance;
		BokehDistance = max(0.0, BokehDistance - Radius);

		if (BokehDistance > 0)
		{
			const float2 Tangent = float2(Normal.y, -Normal.x);

			const float Intensity = pow(BokehDistance, PetzvalFalloffPower);
			const float Stretch = 1 + abs(Petzval) * Intensity;

			float2x2 ToSaggitalSpace = { Normal.x, Normal.y, Tangent.x, Tangent.y };

			bool bMask = Petzval > 0;
			float XStretch =  bMask ? Stretch : 1.0; // Sagittal
			float YStretch = !bMask ? Stretch : 1.0; // Tangential
			float2x2 StretchTransform = { 
				XStretch, 0,
				0, YStretch
			};

			Transform = mul(mul(ToSaggitalSpace, StretchTransform), transpose(ToSaggitalSpace));
		}
	}
#endif //CONFIG_PETZVAL
	return Transform;
}

void ScatterMainVS(
	in uint VertexId : SV_VertexID,
	in uint InstanceId : SV_InstanceID,
	out nointerpolation float4 BokehCenterScreenAndClipPos : TEXCOORD0,
	out nointerpolation float4 ColorAndAbsCocRadius0 : TEXCOORD1,
	out nointerpolation float4 ColorAndAbsCocRadius1 : TEXCOORD2,
	out nointerpolation uint3 GlobalSpriteIdAndPetzvalTransformPacked : TEXCOORD3,
#if TAYLOR_EXPENDED_COC_INTERSECTION
	out nointerpolation float4 TaylorExpensionMAD0   : TEXCOORD4,
	out nointerpolation float4 TaylorExpensionMAD1   : TEXCOORD5,
#endif
	out float4 OutPosition : SV_POSITION)
{
	ResolvedView = ResolveView();

	#if SCATTERING_GROUP_PACKING
		// Number scattering group that can be packed per instance.
		const uint ScatteringGroupPerInstance = VERTEX_SHADER_INVOCATION_TARGET / VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP;

		uint LocalSpriteId = VertexId / VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP;
		uint SpriteVertexId = VertexId - LocalSpriteId * VERTEX_SHADER_INVOCATION_PER_SCATTERING_GROUP;
		uint GlobalSpriteId = ScatteringGroupPerInstance * InstanceId + LocalSpriteId;

	#else //!SCATTERING_GROUP_PACKING
		uint SpriteVertexId = VertexId;
		uint GlobalSpriteId = InstanceId;

	#endif

	// Fetches color.
	float4 Color[PIXEL_COUNT_PER_SCATTER_GROUP];
	float AbsCocRadius[PIXEL_COUNT_PER_SCATTER_GROUP];
	float2 TaylorExpension[PIXEL_COUNT_PER_SCATTER_GROUP];
	float MaxAbsCocRadius = 0;
	
	UNROLL
	for (uint i = 0; i < PIXEL_COUNT_PER_SCATTER_GROUP; i++)
	{
		uint ScatterDrawListOffset = SCATTER_DRAW_LIST_GROUP_STRIDE * GlobalSpriteId + 1 + (i * SCATTER_DRAW_LIST_DATA_PER_PIXEL);
		float4 E = ScatterDrawList[ScatterDrawListOffset];
		
		Color[i] = float4(E.rgb, 0);
		AbsCocRadius[i] = E.w;

		if (i == 0)
			MaxAbsCocRadius = AbsCocRadius[i];
		else
			MaxAbsCocRadius = max(MaxAbsCocRadius, AbsCocRadius[i]);

		/** Taylor expension of order 1 of the original intersection equation:
		 *     I(x = d^2) = M - sqrt(x)
		 *
		 *  With:
		 *     d = PixelDistance from center of the bokeh
		 *
		 *  I(x) ~= - x * 0.5 / sqrt(K) + (M - 0.5 * sqrt(K))
		 *    (x ~= K)
		 *
		 *  K=AbsCocRadius^2
		 */
		{
			float M = AbsCocRadius[i] + ANTI_ALIASING_FEATHER_OFFSET;
			float K = AbsCocRadius[i] * AbsCocRadius[i];
			
			// TaylorExpension[i].x = -0.5 / sqrt(K);
			// TaylorExpension[i].y = M - 0.5 * sqrt(K);
			
			TaylorExpension[i].x = -0.5 * rcp(AbsCocRadius[i]);
			TaylorExpension[i].y = 0.5 * AbsCocRadius[i] + ANTI_ALIASING_FEATHER_OFFSET;
		}
	}

	// Output interpolators.
	const float2 BokehCenterScreenPos = ScatteringScaling * ScatterDrawList[SCATTER_DRAW_LIST_GROUP_STRIDE * GlobalSpriteId].xy;
	const float2 BokehCenterClipPos = (BokehCenterScreenPos * ViewportSize.zw) * 2.0 - 1.0;
	{
		ColorAndAbsCocRadius0 = float4(
			asfloat(PackFloat2ToUInt(Color[0].r, Color[0].g)), 
			asfloat(PackFloat2ToUInt(Color[0].b, AbsCocRadius[0])),
			asfloat(PackFloat2ToUInt(Color[1].r, Color[1].g)), 
			asfloat(PackFloat2ToUInt(Color[1].b, AbsCocRadius[1]))
		);
		ColorAndAbsCocRadius1 = float4(
			asfloat(PackFloat2ToUInt(Color[2].r, Color[2].g)), 
			asfloat(PackFloat2ToUInt(Color[2].b, AbsCocRadius[2])),
			asfloat(PackFloat2ToUInt(Color[3].r, Color[3].g)), 
			asfloat(PackFloat2ToUInt(Color[3].b, AbsCocRadius[3]))
		);
	
		BokehCenterScreenAndClipPos.xy = BokehCenterScreenPos;
		BokehCenterScreenAndClipPos.zw = BokehCenterClipPos;
	
		#if TAYLOR_EXPENDED_COC_INTERSECTION
		{
			TaylorExpensionMAD0.xy = TaylorExpension[0];
			TaylorExpensionMAD0.zw = TaylorExpension[1];
			TaylorExpensionMAD1.xy = TaylorExpension[2];
			TaylorExpensionMAD1.zw = TaylorExpension[3];
		}
		#endif
	}

	GlobalSpriteIdAndPetzvalTransformPacked = 0;
	GlobalSpriteIdAndPetzvalTransformPacked.x = GlobalSpriteId;
#if CONFIG_PETZVAL
	float2x2 PetzvalTransform = CalcPetzvalTransform(BokehCenterClipPos);
	half4 PetzvalTransformHalf = half4(float4(PetzvalTransform));
	GlobalSpriteIdAndPetzvalTransformPacked.yz = uint2(
		PackFloat2ToUInt(PetzvalTransformHalf[0], PetzvalTransformHalf[1]),
		PackFloat2ToUInt(PetzvalTransformHalf[2], PetzvalTransformHalf[3])
	);
#endif

	#if BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_TRIANGLE
		float2 VertexPos = kSpriteVertices[SpriteVertexId];

	#elif BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_INDEXED_TRIANGLE
		float2 VertexPos = float2(SpriteVertexId % 2, SpriteVertexId / 2) * 2 - 1;

	#elif BOKEH_DRAW_METHOD == BOKEH_DRAW_METHOD_RECT
		float2 VertexPos = kSpriteVertices[SpriteVertexId];
		
	#else
		#error Unknown bokeh drawing method.

	#endif

	// The center of the pixel group is 0.5 south east of BokehCenterScreen.
	const float HalfPixelShift = 0.5;

	float2 ViewportUV = ((MaxAbsCocRadius * CocRadiusToCircumscribedRadius * float2(CocInvSqueeze, 1.0) + (HalfPixelShift + ANTI_ALIASING_FEATHER_OFFSET)) * VertexPos + BokehCenterScreenPos + HalfPixelShift) * ViewportSize.zw; 

	OutPosition = float4(ViewportUV.x * 2.0 - 1.0, 1.0 - ViewportUV.y * 2.0, 0.0, 1.0);
}