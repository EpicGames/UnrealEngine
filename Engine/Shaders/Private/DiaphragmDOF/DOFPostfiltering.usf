// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFPostfiltering.usf: Diaphragm DOF's post filtering passes 
=============================================================================*/

#include "DOFCommon.ush"
#include "DOFGatherTileSuggest.ush"
#include "../Random.ush"


#define THREADGROUP_SIZEX		(COC_TILE_SIZE)
#define THREADGROUP_SIZEY		(THREADGROUP_SIZEX)
#define THREADGROUP_TOTALSIZE	(THREADGROUP_SIZEX * THREADGROUP_SIZEY)

uint4 ViewportRect;
float2 MaxInputBufferUV;


float4 ConvolutionInputSize;
Texture2D ConvolutionInput_SceneColor;
Texture2D ConvolutionInput_SeparateAlpha;

Texture2D TileClassification_Foreground;
Texture2D TileClassification_Background;


RWTexture2D<float4>	ConvolutionOutput_SceneColor;
RWTexture2D<float4>	ConvolutionOutput_SeparateAlpha;


static const int2 kOffsets3x3[9] = {
	int2(-1, -1),
	int2( 0, -1),
	int2( 1, -1),
	int2(-1,  0),
	int2( 0,  0),
	int2( 1,  0),
	int2(-1,  1),
	int2( 0,  1),
	int2( 1,  1),
};

// Operator that output, lowest, median and highest values from 3 input values.
void LMHOperator(float4 A, float4 B, float4 C, out float4 L, out float4 M, out float4 H)
{
	// TODO: min3, max3, med3.
	float4 X = min(B, C);
	float4 Y = max(B, C);
	
	L = min(A, X);
	float4 Z = max(A, X);

	M = min(Z, Y);
	H = max(Z, Y);
}

// 3 samples median.
float4 Median(float4 A, float4 B, float4 C)
{
	// TODO: med3.
	float4 L, M, H;
	LMHOperator(A, B, C, L, M, H);
	return M;
}

// 9 samples median using Smith1996.
float4 Median9(float4 In[9])
{
	// First layer.
	float4 L0[3];
	float4 M0[3];
	float4 H0[3];
	for (uint j = 0; j < 3; j++)
	{
		LMHOperator(In[j * 3 + 0], In[j * 3 + 1], In[j * 3 + 2], L0[j], M0[j], H0[j]);
	}
	
	// Second layer.
	// TODO: min3, max3.
	float4 M1[3];
	M1[0] = max(max(L0[0], L0[1]), L0[2]);
	M1[1] = Median(M0[0], M0[1], M0[2]);
	M1[2] = min(min(H0[0], H0[1]), H0[2]);
	
	// Third layer.
	return Median(M1[0], M1[1], M1[2]);
}


// Sample input buffer
float4 SampleInputBuffer(Texture2D Tex, float2 SampleUV)
{
	// Fine to not need a shader permutation here, since the pass is texture bandwidth bound.
	if (true)
	{
		SampleUV = min(SampleUV, MaxInputBufferUV);
	}

	return Tex.SampleLevel(GlobalPointClampedSampler, min(SampleUV, MaxInputBufferUV), 0);
}


// 3x3 neighborhood median sampling.
float4 SampleMedian3x3(Texture2D Tex, float2 TexInvSize, float2 BufferUV)
{
	// Samples the inputs.
	float4 InSamples[9];
	for (uint i = 0; i < 9; i++)
	{
		InSamples[i] = SampleInputBuffer(Tex, BufferUV + float2(kOffsets3x3[i]) * TexInvSize);
	}

	return Median9(InSamples);
}


// 3x3 neighborhood max sampling.
float4 SampleMax3x3(Texture2D Tex, float2 TexInvSize, float2 BufferUV)
{
	float4 Max;
	for (uint i = 0; i < 9; i++)
	{
		float4 Sample = SampleInputBuffer(Tex, BufferUV + float2(kOffsets3x3[i]) * TexInvSize);

		if (i == 0)
		{
			Max = Sample;
		}
		else
		{
			// Shader compilers are smart enough to use max3 here.
			Max = max(Max, Sample);
		}
	}

	return Max;
}


// Postfilter input according to the method permutation dimension.
float4 PostfilterInput(Texture2D Tex, float2 TexInvSize, float2 BufferUV)
{
	#if DIM_POSTFILTER_METHOD == 1 // median
		return SampleMedian3x3(Tex, TexInvSize, BufferUV);

	#elif DIM_POSTFILTER_METHOD == 2 // max
		return SampleMax3x3(Tex, TexInvSize, BufferUV);

	#else
		#error Unknown postfiltering method.

	#endif
}


// Reduce noise of background and foreground layers by taking the 3x3 median.
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void PostfilterMainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID)
{
	ResolvedView = ResolveView();

	float2 InputBufferUV = (DispatchThreadId + 0.5) * ConvolutionInputSize.zw;
	
	BRANCH
	if (any(DispatchThreadId >= ViewportRect.zw))
	{
		return;
	}


	bool bEarlyReturn = false;
	#if DIM_TILE_PERMUTATION
	{
		const uint2 TilePos = GroupId;
		const FCocTileSample CocTileInfos = LoadCocTile(
			COC_TILE_LAYOUT_FGD_SEP_BGD,
			TileClassification_Foreground,
			TileClassification_Background,
			TilePos);

		FGatheringTileSuggestion TileSuggestion = InferGatherTileSuggestion(CocTileInfos, /* LayerProcessing = */ DIM_LAYER_PROCESSING);

		bEarlyReturn = TileSuggestion.bCanEarlyReturn;
	}
	#endif

	BRANCH
	if (bEarlyReturn)
	{
		uint2 OutputPixelPosition = DispatchThreadId;
		float4 Foreground = float4(0, 0, 0, 0);
		float4 Background = float4(0, 0, 0, 0);
		float SeparateAlpha = 0;

		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = Foreground;

		#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			ConvolutionOutput_SceneColor[OutputPixelPosition] = Background;

		#else
			#error Unknown layer processing.
		#endif
		
		#if CONFIG_DOF_ALPHA
			ConvolutionOutput_SeparateAlpha[OutputPixelPosition] = SeparateAlpha;
		#endif
	}
	else
	{
		#if 1 // Lower VGPR footprint.
			uint2 OutputPixelPosition = InputBufferUV * ConvolutionInputSize.xy;
		#else
			uint2 OutputPixelPosition = DispatchThreadId;
		#endif
			
		ConvolutionOutput_SceneColor[OutputPixelPosition] = PostfilterInput(ConvolutionInput_SceneColor, ConvolutionInputSize.zw, InputBufferUV);
		#if CONFIG_DOF_ALPHA
			ConvolutionOutput_SeparateAlpha[OutputPixelPosition] = PostfilterInput(ConvolutionInput_SeparateAlpha, ConvolutionInputSize.zw, InputBufferUV);
		#endif
	}
}
