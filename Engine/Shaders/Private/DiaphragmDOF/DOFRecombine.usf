// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFRecombine.usf: Recombine lower res convolution with full
	res scene color.
=============================================================================*/


#include "DOFCommon.ush"
#include "../SceneTexturesCommon.ush"
#include "../Random.ush"
#include "../MonteCarlo.ush"
#include "../SobolRandom.ush"



//------------------------------------------------------- DEBUG COMPILE TIME CONFIG

// When on, color the output pixels according to how expensive they were.
#define DEBUG_FAST_PATHS 0


//------------------------------------------------------- ENUM VALUES

/** Slight out of focus gathering method. */
	// No slight out of focus.
	#define SLIGHT_FOCUS_METHOD_DISABLED 0

	// Accumulate foreground and background slight out of focus in unique convolution.
	#define SLIGHT_FOCUS_METHOD_UNIQUE_CONVOLUTIONS 1

	// Accumulate foreground and background slight out of focus separatly
	#define SLIGHT_FOCUS_METHOD_SEPARATE_CONVOLUTIONS 2

	
/** Method used to analysis the full resolution neighborhood. */
	// Uses integer atomic
	#define NEIGHBORHOOD_ANALISIS_ATOMIC 0

	// Uses wave instruction.
	#define NEIGHBORHOOD_ANALISIS_WAVE 1
	
	// Uses LDS reduction.
	#define NEIGHBORHOOD_ANALISIS_LDS_REDUCE 2


// Compositing method for the background.
// TODO: shader permutation to scale down.
#define COMPOSITING_METHOD_NONE 0
#define COMPOSITING_METHOD_BILINEAR_BKG 1


//------------------------------------------------------- COMPILE TIME CONFIG

// Configures across layer processing permutations.
#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_FOREGROUND_ONLY
	#define CONFIG_COMPOSITING_METHOD (COMPOSITING_METHOD_NONE)

#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
	#define CONFIG_COMPOSITING_METHOD (COMPOSITING_METHOD_BILINEAR_BKG)
	
#elif DIM_LAYER_PROCESSING == LAYER_PROCESSING_COMBINED
	#define CONFIG_COMPOSITING_METHOD (COMPOSITING_METHOD_BILINEAR_BKG)

#else
	#error Unknown layer processing.
#endif


// Configures across quality permutations.
#if DIM_QUALITY == 0
	#define CONFIG_GATHER_PAIR_COUNT 0
	#define CONFIG_SLIGHT_FOCUS_METHOD (SLIGHT_FOCUS_METHOD_DISABLED)
	#define CONFIG_HOLE_FILLING_METHOD (HOLE_FILLING_METHOD_OPACITY_AMEND)

#elif DIM_QUALITY == 1
	#define CONFIG_GATHER_PAIR_COUNT 12
	#define CONFIG_SLIGHT_FOCUS_METHOD (SLIGHT_FOCUS_METHOD_UNIQUE_CONVOLUTIONS)
	#define CONFIG_HOLE_FILLING_METHOD (HOLE_FILLING_METHOD_SEPARATE_GATHER)
	#define CONFIG_FETCH_FULLRES_COC_FROM_ALPHA (!CONFIG_DOF_ALPHA)

#elif DIM_QUALITY == 2
	#define CONFIG_GATHER_PAIR_COUNT 16
	#define CONFIG_SLIGHT_FOCUS_METHOD (SLIGHT_FOCUS_METHOD_UNIQUE_CONVOLUTIONS)
	#define CONFIG_HOLE_FILLING_METHOD (HOLE_FILLING_METHOD_SEPARATE_GATHER)
	#define CONFIG_FETCH_FULLRES_COC_FROM_ALPHA (!CONFIG_DOF_ALPHA)
	
#else
	#error Unknown quality.
#endif


// Configures the neighborhood analysis method to use for slight out of focus early out.
#if COMPILER_SUPPORTS_WAVE_MINMAX && (PS4_PROFILE || XBOXONE_PROFILE)
	// GCN only optimisation 
	#define NEIGHBORHOOD_ANALISIS_METHOD (NEIGHBORHOOD_ANALISIS_WAVE)
	
#elif COMPILER_HLSLCC
	// Compiler does not like NEIGHBORHOOD_ANALISIS_ATOMIC at all.
	#define NEIGHBORHOOD_ANALISIS_METHOD (NEIGHBORHOOD_ANALISIS_LDS_REDUCE)

#else
	#define NEIGHBORHOOD_ANALISIS_METHOD (NEIGHBORHOOD_ANALISIS_ATOMIC)

#endif

// Clamp full res gathering to have spec hits still temporally stable.
#define CONFIG_CLAMP_FULLRES_GATHER 1

// Clamp buffer UVs
#define CONFIG_CLAMP_DOF_BUFFER_UV 1

// Fetch CoC radius from full res scene color's alpha channel.
#ifndef CONFIG_FETCH_FULLRES_COC_FROM_ALPHA
	#define CONFIG_FETCH_FULLRES_COC_FROM_ALPHA 0
#endif


//------------------------------------------------------- COMPILE TIME CONSTS

// Epsilon used to compare opacity values.
#define OPACITY_EPSILON 0.01


#define GROUP_BORDER_SIZE     (DEFAULT_GROUP_BORDER_SIZE)
#define THREADGROUP_TOTALSIZE (GROUP_BORDER_SIZE * GROUP_BORDER_SIZE)


//------------------------------------------------------- PARAMETERS

float4 ViewportSize;
uint4 ViewportRect;

FScreenTransform DispatchThreadIdToDOFBufferUV;
float2 DOFBufferUVMax;

float4 SeparateTranslucencyBilinearUVMinMax;
uint SeparateTranslucencyUpscaling;
float EncodedCocRadiusToRecombineCocRadius;
float MaxRecombineAbsCocRadius;
float SOFHistoryRejectionBoost;

float4 SceneColorExtentAndInvExtent;
float4 SceneColorViewRectUV;

Texture2D BokehLUT;

Texture2D SceneColorInput;
Texture2D SceneDepthTexture;
Texture2D SceneSeparateCoc;

Texture2D<float> LowResDepthTexture;
Texture2D<float> FullResDepthTexture;

Texture2D SceneSeparateTranslucency;
Texture2D SceneSeparateTranslucencyModulateColor;

float4 ConvolutionInputSize;

Texture2D ForegroundConvolution_SceneColor;
Texture2D ForegroundConvolution_SeparateAlpha;

Texture2D ForegroundHoleFillingConvolution_SceneColor;
Texture2D ForegroundHoleFillingConvolution_SeparateAlpha;

Texture2D SlightOutOfFocusConvolution_SceneColor;
Texture2D SlightOutOfFocusConvolution_SeparateAlpha;

Texture2D BackgroundConvolution_SceneColor;
Texture2D BackgroundConvolution_SeparateAlpha;


RWTexture2D<float4>	SceneColorOutput;

// Utilities to upsample tranmslucency to full resolution
#include "../SeparateTranslucency.ush"
float2 SeparateTranslucencyTextureLowResExtentInverse;

//------------------------------------------------------- INTERMEDIARY STRUCTURES

// Intermediary results of the recombine.
struct FRecombineInputParameters
{
	// The viewport UV of the output pixel.
	float2 ViewportUV;

	// Buffer UV to sample scene texture buffers.
	float2 SceneBufferUV;

	// Buffer UV to sample DOF buffers.
	float2 DOFBufferUV;

	// Buffer size and inv size for the DOF inputs.
	float4 DOFBufferSize;

	// Random seeds
	uint2 Seed0;
};


//------------------------------------------------------- FUNCTIONS

float SampleWorldDepth(float2 BufferUV)
{
	return ConvertFromDeviceZ(SceneDepthTexture.SampleLevel(GlobalPointClampedSampler, BufferUV, 0).x);
}

// {0 to 1} output.
float NoizNorm(float2 N, float X)
{
	N+=X;
	return frac(sin(dot(N.xy,float2(12.9898, 78.233)))*43758.5453);
}

float2 RotVec(float Radius, float Radians)
{
	return Radius * float2(cos(Radians), sin(Radians));
}

void AmendAdditiveColorWithMaxTranslucency(inout float4 Color, inout float Translucency, float MaxTranslucency)
{
	if (Translucency < 1)
	{
		float NewTranslucency = min(Translucency, MaxTranslucency);
		Color *= (1 - NewTranslucency) / (1 - Translucency);
		Translucency = NewTranslucency;
	}
}

// TODO: most of the math below uses CocRadius as expressed in physical pixels, rather than scaling-independent 'encoded' units.
// It's not clear if the math is always accounting for this, as many of these expressions contain magic numbers.


// Compute sample weight according's to its Coc radius.
float ComputeSampleWeight(float CocRadius)
{
	#if 0
		return 1;

	#else
		const float PixelRadius = FullResPixelDistanceToCocDistance(0.5);

		const float MaximumWeight = rcp((4 * PI) * PixelRadius * PixelRadius);

		float Weight = min(rcp((4 * PI) * CocRadius * CocRadius), MaximumWeight);
		
		//Weight /= max(1, CocRadius * 2);


		return Weight;
	#endif
}

float ComputeSampleIntersection(float SampleCocRadius, float SampleDistance)
{
	#if 0 // DEBUG
		return SampleDistance < SampleCocRadius ? 1 : 0;
	#endif

	// Mulitplier is set to 1 / pixel radius = 0.5, and also need a * 2.0 because recombine is done at full resolution.
	const float Multiplier = 4.0;

	// Offset is set to 0.5 / 2 so that when a sample Coc cover half of the pixel (abs(CocRadiusA) - SampleDistance) == 0),
	// we get a 50% overlap.
	const float LinearOffset = 0.5;

	// Minimal Abs coc radius to be considered to avoid SampleCocRadius=0 & SampleDistance=0 returning < 1.0
	const float MinimalAbsCocRadius = 0.25;

	float AbsCocRadius = max(abs(SampleCocRadius), MinimalAbsCocRadius);

	// Compute linear overlap.
	float LinearIntersection = saturate((AbsCocRadius - SampleDistance) * Multiplier + LinearOffset);

	// Pixels are aproximated as disk. So to make the intersection of two disk look better,
	// do a smoothstep.
	return smoothstep(0, 1, LinearIntersection);
}

// Returns the opacity to use to transition to background.
float ComputeBackgroundSampleOpacity(float CocRadius)
{
	//return CocRadius < MAX_RECOMBINE_ABS_COC_RADIUS ? 1 : 0;
	return saturate(MaxRecombineAbsCocRadius - CocRadius);
}

// Returns the opacity to use to transition to background.
float IsConsideredBackgroundSample(float CocRadius)
{
	return ComputeBackgroundSampleOpacity(CocRadius) * saturate((CocRadius - 0.125) * 8);
}

// Compute translucency of the in focus sample.
float ComputeInFocusOpacity(float CocRadius)
{
	// TODO: should be 4*
	return saturate(2 - 4 * abs(CocRadius));
}

// Returns the opacity to use to transition foreground slight out of focus over in focus.
float ComputeForegroundSampleOpacity(float CocRadius)
{
	return saturate(-1 - 8 * CocRadius);
}

// Returns the opacity to use to transition to background.
float IsConsideredForegroundSample(float CocRadius)
{
	return ComputeForegroundSampleOpacity(CocRadius) * saturate(MaxRecombineAbsCocRadius + CocRadius);
}



//------------------------------------------------------- ACCUMULATOR

/** Structs that holds data about a sample for gathering. */
struct FGatherSample
{
	// Sample's scene color (and optionally alpha channel).
	float4 Color;

	// Sample's radius of the Coc
	float CocRadius;

	// Sample's intersection.
	float Intersection;
};

/** Gathering parameters in recombine pass */
struct FFullResGatherParameters
{
	// Radius size in FULL res pixels.
	float KernelPixelRadius;

	// Number of pair of gathering samples.
	uint SamplePairCount;

};

/** Gathering accumulator for recombine pass */
struct FFullResGatherAccumulator
{
	// Parameters of the full res gather.
	FFullResGatherParameters Parameters;

	float4 Color;
	float ColorWeight;

	float Opacity;
	float OpacityWeight;

	uint LayerProcessing;
};

FFullResGatherAccumulator CreateFullResGatherAccumulator(in FFullResGatherParameters GatherParameters)
{
	FFullResGatherAccumulator Accumulator;
	Accumulator.Parameters = GatherParameters;
	
	Accumulator.Color = 0.0;
	Accumulator.ColorWeight = 0.0;
	Accumulator.Opacity = 0.0;
	Accumulator.OpacityWeight = 0.0;
	Accumulator.LayerProcessing = LAYER_PROCESSING_BACKGROUND_ONLY;

	return Accumulator;
}

struct FGatherSampleDerivedParameters
{
	float Weight;
	float IsConsidered;
	float Opacity;
};

FGatherSampleDerivedParameters ComputeSampleDerivates(in FFullResGatherAccumulator Accumulator, in FGatherSample A)
{
	FGatherSampleDerivedParameters DerivedA;
	DerivedA.Weight = ComputeSampleWeight(A.CocRadius);
	
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_ONLY)
	{
		DerivedA.Opacity = ComputeForegroundSampleOpacity(A.CocRadius);
		DerivedA.IsConsidered = IsConsideredForegroundSample(A.CocRadius);
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		// Don't care about weight for hole filling
		DerivedA.Weight = 1;

		DerivedA.Opacity = ComputeForegroundSampleOpacity(A.CocRadius);
		DerivedA.IsConsidered = 1 - IsConsideredForegroundSample(A.CocRadius);
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		DerivedA.Opacity = ComputeBackgroundSampleOpacity(A.CocRadius);
		DerivedA.IsConsidered = saturate(MaxRecombineAbsCocRadius - abs(A.CocRadius));
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_BACKGROUND_ONLY)
	{
		DerivedA.Opacity = ComputeBackgroundSampleOpacity(A.CocRadius);
		DerivedA.IsConsidered = IsConsideredBackgroundSample(A.CocRadius);
	}

	return DerivedA;
}

void HoleFillCloserSample(
	in FFullResGatherAccumulator Accumulator,
	inout FGatherSample A, inout FGatherSampleDerivedParameters DerivedA,
	in FGatherSample Closer, in FGatherSampleDerivedParameters DerivedCloser)
{
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_ONLY)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;

		#if 1 // Used with LAYER_PROCESSING_FOREGROUND_HOLE_FILLING
				
		#elif 0 // looks nice over slight out of focus, but looks bad over large background out of focus.
			Opacity[1] = Opacity[0] * ComputeBackgroundSampleOpacity(S[1].CocRadius);
			IsConsidered[1] = Opacity[1] * IsConsidered[0];

		#else
			DerivedA.IsConsidered = DerivedCloser.IsConsidered;
			DerivedA.Opacity = DerivedCloser.Opacity;
		#endif
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_FOREGROUND_HOLE_FILLING)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;
		
		DerivedA.Opacity = DerivedCloser.Opacity;
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		A.Intersection = Closer.Intersection;
		DerivedA.Weight = DerivedCloser.Weight;
	}
}

void AccumulateSample(
	inout FFullResGatherAccumulator Accumulator,
	in FGatherSample A,
	in FGatherSampleDerivedParameters DerivedA)
{
	float ColorWeight = A.Intersection * DerivedA.Weight * DerivedA.IsConsidered;
	float OpacityWeight = A.Intersection;
	
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_BACKGROUND_ONLY)
	{
		// This works really well to have smaller out ofcus than the gathering kernel.
		DerivedA.Opacity *= DerivedA.Weight * rcp(ComputeSampleWeight(Accumulator.Parameters.KernelPixelRadius * 0.5));
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		//DerivedA.Opacity *= DerivedA.Weight * rcp(ComputeSampleWeight(Accumulator.Parameters.KernelPixelRadius * 0.5));
	}
	else if (1)
	{
		DerivedA.Opacity *= DerivedA.Weight * rcp(ComputeSampleWeight(Accumulator.Parameters.KernelPixelRadius * 0.5));
	}

	Accumulator.Color += ColorWeight * A.Color;
	Accumulator.ColorWeight += ColorWeight;
	
	Accumulator.Opacity += OpacityWeight * DerivedA.Opacity;
	Accumulator.OpacityWeight += OpacityWeight;
}

/** Accumulates mirror samples. */
void AccumulateMirrorSamples(inout FFullResGatherAccumulator Accumulator, in FGatherSample S[2])
{
	FGatherSampleDerivedParameters DerivedS[2];

	UNROLL
	for (uint i = 0; i < 2; i++)
	{
		DerivedS[i] = ComputeSampleDerivates(Accumulator, S[i]);
	}

	// Mirror hole filling.
	#if 1
		if (S[1].CocRadius > S[0].CocRadius)
		{
			HoleFillCloserSample(Accumulator, S[1], DerivedS[1], S[0], DerivedS[0]);
		}
		else if (S[0].CocRadius > S[1].CocRadius)
		{
			HoleFillCloserSample(Accumulator, S[0], DerivedS[0], S[1], DerivedS[1]);
		}
	#else
		if (IsForeground(S[0].CocRadius) && S[1].CocRadius > S[0].CocRadius)
		{
			HoleFillCloserSample(Accumulator, S[1], DerivedS[1], S[0], DerivedS[0]);
		}
		else if (IsForeground(S[1].CocRadius) && S[0].CocRadius > S[1].CocRadius)
		{
			HoleFillCloserSample(Accumulator, S[0], DerivedS[0], S[1], DerivedS[1]);
		}
	#endif
	
	UNROLL
	for (uint j = 0; j < 2; j++)
	{
		AccumulateSample(Accumulator, S[j], DerivedS[j]);
	}
}

/** Accumulates center sample. */
void AccumulateCenterSample(inout FFullResGatherAccumulator Accumulator, in FGatherSample A)
{
	FGatherSampleDerivedParameters DerivedA = ComputeSampleDerivates(Accumulator, A);

	// Force this sample to be considered to guareentee their is a resolved color if in focus or background.
	DerivedA.IsConsidered = 1;

	AccumulateSample(Accumulator, A, DerivedA);
}

/** Resolves the slightly out of focus. */
void ResolveAccumulator(
	in FFullResGatherAccumulator Accumulator,
	out float4 OutGatherBackgroundUnpremultipliedColor,
	out float OutGatherBackgroundOpacity)
{
	const float SampleCount = 1.0 + 2.0 * Accumulator.Parameters.SamplePairCount;

	float Opacity;
	if (Accumulator.LayerProcessing == LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS)
	{
		//Opacity = saturate(Accumulator.Opacity * SafeRcp(SampleCount));
		Opacity = saturate(Accumulator.Opacity * SafeRcp(Accumulator.OpacityWeight));
	}
	else if (Accumulator.LayerProcessing == LAYER_PROCESSING_BACKGROUND_ONLY || 1)
	{
		Opacity = saturate(Accumulator.Opacity * SafeRcp(Accumulator.OpacityWeight));
	}
	else
	{
		Opacity = saturate(Accumulator.Opacity * SafeRcp(SampleCount));
	}

	OutGatherBackgroundOpacity = Accumulator.ColorWeight > 0 ? Opacity : 0;
	OutGatherBackgroundUnpremultipliedColor = Accumulator.Color * (SafeRcp(Accumulator.ColorWeight));
}


//------------------------------------------------------- KERNEL

#if CONFIG_SLIGHT_FOCUS_METHOD != SLIGHT_FOCUS_METHOD_DISABLED

void FetchAndAccumulateSamplePair(
	in const FRecombineInputParameters InputParameters,
	in float2 PixelOffset,
	inout FFullResGatherAccumulator Accumulator)
{
	// Accuratly quantize sample offset so the intersection get evaluated at the center,
	// unless using a look up table to know SampleDistance.
	#if DIM_BOKEH_SIMULATION == BOKEH_SIMULATION_DISABLED
		PixelOffset = sign(PixelOffset) * floor(abs(PixelOffset) + 0.5);
	#endif

	// Distance of the sample from output pixels in half res pixel unit.
	float SampleDistance = FullResPixelDistanceToCocDistance(length(PixelOffset));

	// Scene buffer offset.
	float2 SceneBufferUVOffset = PixelOffset * SceneColorExtentAndInvExtent.zw * float2(CocInvSqueeze, 1.0);
	
	// Two sample to gather at same time.
	FGatherSample Sample[2];
	
	UNROLL
	for (uint k = 0; k < 2; k++)
	{
		const float SampleSign = (k == 0) ? 1.0 : -1.0;

		// Fetch SampleDistance from lookup table.
		#if DIM_BOKEH_SIMULATION != BOKEH_SIMULATION_DISABLED
		if (k == 0 || DIM_BOKEH_SIMULATION == BOKEH_SIMULATION_GENERAL)
		{
			const float InvLutSize = rcp(float(BOKEH_LUT_SIZE));

			float2 LookupUV = (0.5 + 0.5 * InvLutSize) + PixelOffset * (SampleSign * InvLutSize) * float2(CocInvSqueeze, 1.0);
			float4 LookupSample = BokehLUT.SampleLevel(GlobalPointClampedSampler, LookupUV, 0);
			SampleDistance = LookupSample.x;
		}
		#endif

		float2 BufferUV = InputParameters.SceneBufferUV + SampleSign * SceneBufferUVOffset;
		BufferUV = clamp(BufferUV, SceneColorViewRectUV.xy, SceneColorViewRectUV.zw);
		
		// Fetch full res color and CocRadius.
		#if CONFIG_FETCH_FULLRES_COC_FROM_ALPHA
			Sample[k].Color = SceneColorInput.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
			Sample[k].CocRadius = Sample[k].Color.a * EncodedCocRadiusToRecombineCocRadius;
		#else
			Sample[k].Color = SceneColorInput.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
			Sample[k].CocRadius = SceneDepthToCocRadius(SampleWorldDepth(BufferUV));
		#endif
		
		// Convert scene color alpha from translucency to opacity.
		Sample[k].Color.a = 1 - Sample[k].Color.a;

		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			Sample[k].CocRadius = max(Sample[k].CocRadius, 0);
		#endif

		Sample[k].Intersection = ComputeSampleIntersection(Sample[k].CocRadius, SampleDistance);
	}

	AccumulateMirrorSamples(Accumulator, Sample);
}

void GatherToAccumulator(
	in const FRecombineInputParameters InputParameters,
	in const FFullResGatherParameters GatherParameters,
	inout FFullResGatherAccumulator Accumulator)
#if 0 // brute force the gathering kernel.
{
	int QuadSize = 0 * 2 * MAX_RECOMBINE_ABS_COC_RADIUS;

	UNROLL
	for (int x = -QuadSize; x <= QuadSize; x++)
	UNROLL
	for (int y = 0; y <= QuadSize; y++)
	{
		if (y == 0 && x <= 0)
		{
			continue;
		}

		const float2 PixelOffset = float2(x, y);
		const float PixelDistance = length(PixelOffset);

		if (PixelDistance > QuadSize)
		{
			continue;
		}
		
		FetchAndAccumulateSamplePair(InputParameters, PixelOffset, Accumulator);
	}
}
#else
{
	// Samples at full resolution.
	LOOP
	for(uint SamplePairId = 0; SamplePairId < GatherParameters.SamplePairCount; SamplePairId++)
	{
		float2 E = Hammersley16(SamplePairId, CONFIG_GATHER_PAIR_COUNT, InputParameters.Seed0);
		float2 DiskRandom = UniformSampleDiskConcentricApprox(E);
	
		float2 PixelOffset = GatherParameters.KernelPixelRadius * DiskRandom;
		
		// We already sampled the center pixels, and there is no point sampling it again with very small Coc.
		// Therefore clipped the offset so that it does not sample the center again.
		//FLATTEN
		if (any(abs(PixelOffset) <= 0.5) && 0)
		{
			PixelOffset = clamp(PixelOffset * SafeRcp(max(abs(PixelOffset.x), abs(PixelOffset.y))), -1, 1);
		}

		FetchAndAccumulateSamplePair(InputParameters, PixelOffset, Accumulator);
	}
}
#endif

#endif // CONFIG_SLIGHT_FOCUS_METHOD != SLIGHT_FOCUS_METHOD_DISABLED


//------------------------------------------------------- ENTRY POINT

#if NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_ATOMIC

groupshared uint SharedMaxConsideredAbsCocRadius;


#elif NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_LDS_REDUCE

groupshared float SharedMaxConsideredAbsCocRadius[GROUP_BORDER_SIZE * GROUP_BORDER_SIZE];


#endif

[numthreads(GROUP_BORDER_SIZE, GROUP_BORDER_SIZE, 1)]
void RecombineMainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex) 
{
	ResolvedView = ResolveView();

	float4 Debug = 0;

	// Setup input parameters.
	FRecombineInputParameters InputParameters;
	{
		InputParameters.DOFBufferSize = ConvolutionInputSize;

		InputParameters.ViewportUV = (DispatchThreadId + 0.5) * ViewportSize.zw;

		InputParameters.SceneBufferUV = SceneColorViewRectUV.xy + InputParameters.ViewportUV * (SceneColorViewRectUV.zw - SceneColorViewRectUV.xy);
		InputParameters.SceneBufferUV = clamp(InputParameters.SceneBufferUV, SceneColorViewRectUV.xy, SceneColorViewRectUV.zw);

		// - 0.5 * TemporalJitterPixels because DOF buffer is non temporally jittering, thanks to half res TAA pass.
		InputParameters.DOFBufferUV = ApplyScreenTransform(float2(DispatchThreadId), DispatchThreadIdToDOFBufferUV);

		if (CONFIG_CLAMP_DOF_BUFFER_UV)
		{
			InputParameters.DOFBufferUV = min(InputParameters.DOFBufferUV, DOFBufferUVMax);
		}

		InputParameters.Seed0 = Rand3DPCG16(int3(DispatchThreadId, View.StateFrameIndexMod8)).xy;
	}

	//Fetch foreground layer first to early return if ForegroundTranslucency == 0.0.
	float4 ForegroundColor;
	float ForegroundTranslucency;
	{
		#if DIM_LAYER_PROCESSING == LAYER_PROCESSING_BACKGROUND_ONLY
			ForegroundColor = 0;
			ForegroundTranslucency = 1;

		#elif CONFIG_DOF_ALPHA
			// Sample premultiplied RGBA foreground.
			ForegroundColor = ForegroundConvolution_SceneColor.SampleLevel(
				GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0);
			ForegroundTranslucency = 1 - ForegroundConvolution_SeparateAlpha.SampleLevel(
				GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0).r;
		#else
			// Sample premultiplied RGBA foreground.
			ForegroundColor = ForegroundConvolution_SceneColor.SampleLevel(
				GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0);
			ForegroundTranslucency = 1 - ForegroundColor.a;

		#endif
	}

	
	// Get full res color and coc radius.
	FGatherSample CenterSample;
	CenterSample.Color = SceneColorInput.SampleLevel(GlobalPointClampedSampler, InputParameters.SceneBufferUV, 0);
	CenterSample.Intersection = 1.0;

	#if CONFIG_FETCH_FULLRES_COC_FROM_ALPHA
		CenterSample.CocRadius = CenterSample.Color.a * EncodedCocRadiusToRecombineCocRadius;
	#else
		CenterSample.CocRadius = SceneDepthToCocRadius(SampleWorldDepth(InputParameters.SceneBufferUV));
	#endif
	
	// Convert scene color alpha from translucency to opacity.
	CenterSample.Color.a = 1 - CenterSample.Color.a;

	// Whether can display solly foreground.
	bool bCanReturnForegroundOnly = ForegroundTranslucency < OPACITY_EPSILON;
	
	// Group constant: Whether should do full resolution gathering for slight out of focus.
	bool bGatherFullRes = false;

	#if CONFIG_SLIGHT_FOCUS_METHOD != SLIGHT_FOCUS_METHOD_DISABLED
	// Full resolution gather's parameters.
	FFullResGatherParameters GatherParameters;
	{
		#if NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_ATOMIC
		{
			SharedMaxConsideredAbsCocRadius = 0;
			GroupMemoryBarrierWithGroupSync();
		}
		#endif


		// Grab the smallest slightly out of focus Coc radius of the tile.
		float TileMaxConsideredAbsCocRadius;
		{
			float MaxConsideredAbsCocRadius = abs(CenterSample.CocRadius) < MaxRecombineAbsCocRadius ? abs(CenterSample.CocRadius) : 0;

			for (uint j = 0; j < 4; j++)
			{
				const float2 SamplePixelOffset = float2(kOffsetsCross3x3[j]) * CocDistanceToFullResPixelDistance(MaxRecombineAbsCocRadius);

				float2 SampleUVOffset = SceneColorExtentAndInvExtent.zw * SamplePixelOffset;
				float2 SampleUV = InputParameters.SceneBufferUV + SampleUVOffset;
				SampleUV = clamp(SampleUV, SceneColorViewRectUV.xy, SceneColorViewRectUV.zw);

				#if CONFIG_FETCH_FULLRES_COC_FROM_ALPHA
					float SampleCocRadius = SceneColorInput.SampleLevel(GlobalPointClampedSampler, SampleUV, 0).a * EncodedCocRadiusToRecombineCocRadius;
				#else
					float SampleCocRadius = SceneDepthToCocRadius(SampleWorldDepth(SampleUV));
				#endif

				float SampleAbsCocRadius = abs(SampleCocRadius);

				#if 0
				// Compute the minimum CocRadius to overlap with the group's tile, to reduce amount of tiles gathering uselessly.
				// TODO: have not witnessed any performance regression or improvement with this yet.
				{
					// Compute the minimum CocRadius to overlap with the group's tile.
					float2 ThreadDistanceToGroupBorder = lerp(GroupThreadId, (GROUP_BORDER_SIZE - 1) - GroupThreadId, kSquare2x2[j]);
					float2 OutsideGroupPixelOffset = abs(SamplePixelOffset) - ThreadDistanceToGroupBorder;
					float2 OutsideGroupCocOffset = FullResPixelDistanceToCocDistance(OutsideGroupPixelOffset);

					float MinCocRadiusSquare = dot(OutsideGroupCocOffset, OutsideGroupCocOffset);

					// Not interested if the CocRadius is too large, or does not overlap with the group's tile.
					if (SampleAbsCocRadius < MaxRecombineAbsCocRadius &&
						SampleAbsCocRadius * SampleAbsCocRadius > MinCocRadiusSquare)
					{
						MaxConsideredAbsCocRadius = max(MaxConsideredAbsCocRadius, SampleAbsCocRadius);
					}
				}
				#else
				{
					MaxConsideredAbsCocRadius = max(MaxConsideredAbsCocRadius, SampleAbsCocRadius < MaxRecombineAbsCocRadius ? SampleAbsCocRadius : MaxConsideredAbsCocRadius);
				}
				#endif
			}

			#if NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_ATOMIC
			{
				// Do atomic min and max of the positive or null float MaxConsideredAbsCocRadius
				// as if they were uint.
				uint Unused;
				InterlockedMax(SharedMaxConsideredAbsCocRadius, asuint(MaxConsideredAbsCocRadius), Unused);

				GroupMemoryBarrierWithGroupSync();

				// Read atomic counters.
				TileMaxConsideredAbsCocRadius = asfloat(SharedMaxConsideredAbsCocRadius);
			}
			#elif NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_LDS_REDUCE
			{
				SharedMaxConsideredAbsCocRadius[GroupThreadIndex] = MaxConsideredAbsCocRadius;
				GroupMemoryBarrierWithGroupSync();
				
				// Safe for vector sizes 32 or larger, AMD and NV
				// TODO Intel variable size vector
				UNROLL
				for (uint i = 0; i < 5; i++)
				{
					const uint ReduceSize = 32u >> i;
					if (GroupThreadIndex < ReduceSize)
					{
						MaxConsideredAbsCocRadius = max(MaxConsideredAbsCocRadius, SharedMaxConsideredAbsCocRadius[GroupThreadIndex + ReduceSize]);
						SharedMaxConsideredAbsCocRadius[GroupThreadIndex] = MaxConsideredAbsCocRadius;
					}
				}

				TileMaxConsideredAbsCocRadius = SharedMaxConsideredAbsCocRadius[0];
			}
			#elif NEIGHBORHOOD_ANALISIS_METHOD == NEIGHBORHOOD_ANALISIS_WAVE
			{
				TileMaxConsideredAbsCocRadius = WaveActiveMax(MaxConsideredAbsCocRadius);
			}
			#else
				#error Unknown neighborhood analisis method to use.
			#endif
		}

		// Determines what should be done.
		{
			// Gather at full resolution only if we know there is considered neighborhood that a COC radius big enough.
			bGatherFullRes = TileMaxConsideredAbsCocRadius > 0.125;

			// No need to gather at full res for this pixel if totally occluded by foreground.
			bGatherFullRes = bGatherFullRes && !bCanReturnForegroundOnly;
		}

		// Set up gathering parameters.
		{
			float FullResKernelRadius = CocDistanceToFullResPixelDistance(TileMaxConsideredAbsCocRadius);

			// Set the size of the kernel to exactly the max convolution that needs to be done.
			GatherParameters.KernelPixelRadius = ceil(FullResKernelRadius);

			// Increase the size of the kernel radius to avoid the gather Point sampler to create Coc step artifacts.
			GatherParameters.KernelPixelRadius += 0.5;

			float KenelArea = Pow2(FullResKernelRadius) * PI;

			float RecommendedPairCount = KenelArea * 0.5;

			// Number of pair of sample.
			GatherParameters.SamplePairCount = min(uint(CONFIG_GATHER_PAIR_COUNT), uint(RecommendedPairCount));
		}
	}
	#endif // CONFIG_SLIGHT_FOCUS_METHOD != SLIGHT_FOCUS_METHOD_DISABLED
	
	if (any((ViewportRect.xy + DispatchThreadId) >= ViewportRect.zw))
	{
		return;
	}

	float GatherBackgroundOpacity = ComputeInFocusOpacity(CenterSample.CocRadius);
	float4 GatherBackgroundUnpremultipliedColor = CenterSample.Color;

	#if CONFIG_SLIGHT_FOCUS_METHOD == SLIGHT_FOCUS_METHOD_DISABLED
	{
		GatherBackgroundOpacity = ComputeBackgroundSampleOpacity(CenterSample.CocRadius);
	}
	#elif CONFIG_SLIGHT_FOCUS_METHOD == SLIGHT_FOCUS_METHOD_UNIQUE_CONVOLUTIONS
	BRANCH
	if (bGatherFullRes)
	{
		// Full resolution's opacity with background.
		FFullResGatherAccumulator Accumulator = CreateFullResGatherAccumulator(GatherParameters);
		Accumulator.LayerProcessing = LAYER_PROCESSING_SLIGHT_OUT_OF_FOCUS;

		// Accumulate center sample first to reduce VGPR pressure.
		AccumulateCenterSample(Accumulator, CenterSample);
			
		// TODO: Adaptive number of sample.
		GatherToAccumulator(
			InputParameters, GatherParameters,
			Accumulator);

		// The full resolution gathering kernel is sampling directly the full res scene color, that is jittering
		// and potentially flickering on spec hits. To avoids issues with TAA's clamping box, we clamp this with the
		// with prefiltering scene color for temporal stability.
		// 
		// TODO: should this be done in YCoCg or LCoCg?
		#if CONFIG_CLAMP_FULLRES_GATHER
		{
			float4 Min;
			float4 Max;

			float2 ClampUVBox = InputParameters.DOFBufferSize.zw;

			UNROLL
			for (uint i = 0; i < 4; i++)
			{
				float2 SampleUV = InputParameters.DOFBufferUV + (0.5 * kOffsetsCross3x3[i]) * ClampUVBox;

				if (CONFIG_CLAMP_DOF_BUFFER_UV)
				{
					SampleUV = min(SampleUV, DOFBufferUVMax);
				}

				float4 StableSampleColor = SlightOutOfFocusConvolution_SceneColor.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);

				if (i == 0)
				{
					Min = Max = StableSampleColor;
				}
				else
				{
					Min = min(Min, StableSampleColor);
					Max = max(Max, StableSampleColor);
				}
			}

			// TODO: Increase constrast of limit a little to workaround to strong denoise at near-in-focus (stolen from CircleDOF).
			#if 1
			{
				float4 HD = Min;
				float Small = SOFHistoryRejectionBoost * (1.0 - saturate(CenterSample.CocRadius * CenterSample.CocRadius * rcp(64.0)));
				Max += HD * Small;
				Min -= HD * Small;

				// Ensures the temporally stable opacity remains between 0-1.
				// Uses saturate() instead of min(0 and max() to be optimised as
				// saturate() MAD post modifier on GCN.
				Min.a = saturate(Min.a);
				Max.a = saturate(Max.a);
			}
			#endif

			float ClampWeight = saturate(CenterSample.CocRadius * CenterSample.CocRadius * 4.0);

			// Clamp color
			float3 ClampedColor = clamp(Accumulator.Color.rgb, Min.rgb * Accumulator.ColorWeight, Max.rgb * Accumulator.ColorWeight);

			Accumulator.Color.rgb = lerp(Accumulator.Color.rgb, ClampedColor, ClampWeight);

			// Clamp opacity.			
			float ClampedOpacity = clamp(Accumulator.Opacity, Min.a * Accumulator.OpacityWeight, Max.a * Accumulator.OpacityWeight);

			Accumulator.Opacity = lerp(Accumulator.Opacity, ClampedOpacity, ClampWeight);
		}
		#endif

		// Resolve full res gather.
		ResolveAccumulator(Accumulator, GatherBackgroundUnpremultipliedColor, GatherBackgroundOpacity);
	}
	#endif
	
	// Compose lower res foreground with full res gather foreground.
	float4 GatherForegroundAdditiveColor = ForegroundColor;
	float GatherForegroundTranslucency = ForegroundTranslucency;

	// Sample lower res background, if necessary.
	float4 BackgroundColor = 0.0;
	float BackgroundValidity = 0.0;

	// Separate foregroung hole filling, exposed mainly for debugging purposes.
	float4 HoleFillingAdditiveColor = 0;
	float HoleFillingTranslucency = 1;

	BRANCH
	if ((GatherForegroundTranslucency < OPACITY_EPSILON || bCanReturnForegroundOnly) && 0)
	{
		GatherForegroundAdditiveColor *= SafeRcp(1 - GatherForegroundTranslucency);
		GatherForegroundTranslucency = 0;
	}
	else
	{
		#if CONFIG_COMPOSITING_METHOD == COMPOSITING_METHOD_BILINEAR_BKG
		{
			BackgroundColor = BackgroundConvolution_SceneColor.SampleLevel(GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0);

			#if CONFIG_DOF_ALPHA
				BackgroundValidity = BackgroundConvolution_SeparateAlpha.SampleLevel(GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0).r;
			#else
				//Background = float4(0, 0, 0, 1);
				BackgroundValidity = BackgroundColor.a;
			#endif
	
			// Make sure the background color is always normalized, or unrendered.
			BackgroundColor *= SafeRcp(BackgroundValidity);
		}
		#endif

		// Hole fill the background in output final scene color before composing foreground on top.
		#if CONFIG_HOLE_FILLING_METHOD == HOLE_FILLING_METHOD_SEPARATE_GATHER
		{
			HoleFillingAdditiveColor = ForegroundHoleFillingConvolution_SceneColor.SampleLevel(GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0);

			#if CONFIG_DOF_ALPHA
				HoleFillingTranslucency = ForegroundHoleFillingConvolution_SeparateAlpha.SampleLevel(GlobalBilinearClampedSampler, InputParameters.DOFBufferUV, 0).r;
			#else
				HoleFillingTranslucency = HoleFillingAdditiveColor.a;
			#endif

			float MaxTranslucency = 1 - IsConsideredForegroundSample(CenterSample.CocRadius);

			// Force the hole filling translucency to 0 if the background is unrendered.
			if (BackgroundValidity <= 0.001)
			{
				if (HoleFillingTranslucency < 1.0)
				{
					float MaxTranslucency = 1 - IsConsideredForegroundSample(CenterSample.CocRadius);

					MaxTranslucency = min(MaxTranslucency, BackgroundValidity);
					AmendAdditiveColorWithMaxTranslucency(HoleFillingAdditiveColor, HoleFillingTranslucency, MaxTranslucency);
				}
				else
				{
					GatherBackgroundOpacity = 1;
				}
			}

			BackgroundColor = BackgroundColor * HoleFillingTranslucency + HoleFillingAdditiveColor;
		}
		#else
		if (BackgroundValidity <= 0.001)
		{
			GatherBackgroundOpacity = 1;
		}
		#endif
	}

	// Compose background loweer res gather and full res gather.
	float4 OutputFinalSceneColor = BackgroundColor * (1 - GatherBackgroundOpacity) + GatherBackgroundOpacity * GatherBackgroundUnpremultipliedColor;
	
	// Forces foreground translucency to 0 when large out of focus high res foreground, to
	// avoid background leaking.
	#if CONFIG_HOLE_FILLING_METHOD == HOLE_FILLING_METHOD_OPACITY_AMEND
	if (GatherForegroundTranslucency < 1 && BackgroundValidity != 1.0)
	{
		#if CONFIG_GATHER_PAIR_COUNT == 0
			float MaxTranslucency = saturate(MaxRecombineAbsCocRadius + CenterSample.CocRadius);
		#else
			float MaxTranslucency = 1 - ComputeForegroundSampleOpacity(CenterSample.CocRadius);
		#endif
		
		AmendAdditiveColorWithMaxTranslucency(GatherForegroundAdditiveColor, GatherForegroundTranslucency, MaxTranslucency);
	}
	#endif
	
	// Compose foreground.
	OutputFinalSceneColor = OutputFinalSceneColor * GatherForegroundTranslucency + GatherForegroundAdditiveColor;

	// Compose separate translucency.
	if (1)
	{
		NearestDepthNeighborUpsamplingResult UpsampleResult;
		if (SeparateTranslucencyUpscaling == 0)
		{
			UpsampleResult.bUsePointSampler = true;
			UpsampleResult.UV = InputParameters.SceneBufferUV;
		}
		else
		{
			float2 PixelPos = (View.ViewRectMin.xy + DispatchThreadId) + 0.5;
			UpsampleResult = NearestDepthNeighborUpsampling(
				LowResDepthTexture,
				FullResDepthTexture,
				PixelPos,
				InputParameters.SceneBufferUV,
				SeparateTranslucencyTextureLowResExtentInverse);
		}
		
		UpsampleResult.UV = clamp(UpsampleResult.UV, SeparateTranslucencyBilinearUVMinMax.xy, SeparateTranslucencyBilinearUVMinMax.zw);

		float4 SeparateTranslucencyColor = 0;
		float4 SeparateTranslucencyModulateColor = 0;
		if (UpsampleResult.bUsePointSampler)
		{
			SeparateTranslucencyColor = SceneSeparateTranslucency.SampleLevel(GlobalPointClampedSampler, UpsampleResult.UV, 0);
			SeparateTranslucencyModulateColor = SceneSeparateTranslucencyModulateColor.SampleLevel(GlobalPointClampedSampler, UpsampleResult.UV, 0);
		}
		else
		{
			SeparateTranslucencyColor = SceneSeparateTranslucency.SampleLevel(GlobalBilinearClampedSampler, UpsampleResult.UV, 0);
			SeparateTranslucencyModulateColor = SceneSeparateTranslucencyModulateColor.SampleLevel(GlobalBilinearClampedSampler, UpsampleResult.UV, 0);
		}
		float SeparateTranslucencyBackgroundVisibility = SeparateTranslucencyColor.a;
		float GreyScaleModulateColorBackgroundVisibility = dot(SeparateTranslucencyModulateColor.rgb, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));

		// This matches what is done in ComposeSeparateTranslucency.usf
		OutputFinalSceneColor.rgb = OutputFinalSceneColor.rgb * SeparateTranslucencyBackgroundVisibility * SeparateTranslucencyModulateColor.rgb + SeparateTranslucencyColor.rgb;
		// Also stores BackgroundVisibility (=transmittance) in alpha
		float FinalSceneVisibility = 1.0 - OutputFinalSceneColor.a;
		OutputFinalSceneColor.a = FinalSceneVisibility * SeparateTranslucencyBackgroundVisibility * GreyScaleModulateColorBackgroundVisibility;

		// Convert from visibility to coverage to comply with the following process
		OutputFinalSceneColor.a = 1.0f - OutputFinalSceneColor.a;
	}

	// Convert alpha channel from opacity back to translucency.
	OutputFinalSceneColor.a = 1 - OutputFinalSceneColor.a;

	// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
	// (0.005 chosen to accomodate handling of 1/255)
	#if CONFIG_DOF_ALPHA
		OutputFinalSceneColor.a = select(OutputFinalSceneColor.a < 0.005, 0.0, OutputFinalSceneColor.a);
		OutputFinalSceneColor.a = select(OutputFinalSceneColor.a > 0.995, 1.0, OutputFinalSceneColor.a);
	#endif
	
	// Debug optimisation colors.
	#if 0
	{
		float3 DebugColor;

		if (bGatherFullRes)
		{
			// RED: Full res gather.
			Debug = float4(1.0, 0.0, 0.0, 0.0);
		}
		else if (bCanReturnForegroundOnly)
		{
			// GREEN: Foreground is the cheapest.
			Debug = float4(0.0, 1.0, 0.0, 0.0);
		}
		else
		{
			// BLUE: Fetch foreground and background.
			Debug = float4(0.0, 0.0, 1.0, 0.0);
		}
	}
	#elif 0
	{
		if (bGatherFullRes)
		{
			Debug = float4(1.0, 0.0, 0.0, 0.0);
		}
		else
		{
			Debug = float4(0.0, 1.0, 0.0, 0.0);
		}
	}
	#endif
	
	uint2 OutputPixelPosition = ViewportRect.xy + DispatchThreadId;

	#if CONFIG_DOF_ALPHA
		SceneColorOutput[OutputPixelPosition] = OutputFinalSceneColor;
	#else
		SceneColorOutput[OutputPixelPosition] = float4(OutputFinalSceneColor.rgb, 0);
	#endif
		

	#if DEBUG_OUTPUT
	{
		DebugOutput[OutputPixelPosition] = Debug;
	}
	#endif
}
