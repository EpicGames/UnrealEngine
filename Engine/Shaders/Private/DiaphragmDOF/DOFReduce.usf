// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFReduce.usf: Diaphragm DOF's reduce pass. 
=============================================================================*/

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "DOFDownsample.ush"
#include "DOFHybridScatterCompilation.ush"
#include "DOFVignetteCommon.ush"
#include "../ReductionCommon.ush"


//------------------------------------------------------- ENUM VALUES

/** Method used to allocate scatter group globally. */
	// Uses bit or atomic
	#define SCATTER_ALLOC_METHOD_ATOMIC 0

	// Uses wave bit or instruction.
	#define SCATTER_ALLOC_METHOD_WAVE 1
	
	// Uses LDS entry for each thread.
	#define SCATTER_ALLOC_METHOD_LDS 2


//------------------------------------------------------- COMPILE TIME CONSTANTS

#define SHARED_REDUCE_COUNT 3

#define OUTPUT_MIP_COUNT (SHARED_REDUCE_COUNT + 1)

#define THREADGROUP_TILE_SIZE (1 << SHARED_REDUCE_COUNT)
#define THREADGROUP_TOTALSIZE (THREADGROUP_TILE_SIZE * THREADGROUP_TILE_SIZE)

#if CONFIG_DOF_ALPHA
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_ALPHA_COC)
#elif DIM_RGB_COLOR_BUFFER
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC)
#else
	#define CONFIG_GATHER_INPUT_LAYOUT (GATHER_INPUT_LAYOUT_RGB_COC)
#endif


// Configures the neighborhood analysis method to use for slight out of focus early out.
#if COMPILER_SUPPORTS_WAVE_BIT_ORAND && (COMPILER_PSSL || XBOXONE_PROFILE)
	// GCN only optimisation 
	#define SCATTER_ALLOC_METHOD (SCATTER_ALLOC_METHOD_WAVE)
	
#elif COMPILER_HLSLCC
	// Compiler does not like InterlockedOr().
	#define SCATTER_ALLOC_METHOD (SCATTER_ALLOC_METHOD_LDS)

#else
	#define SCATTER_ALLOC_METHOD (SCATTER_ALLOC_METHOD_ATOMIC)

#endif



//------------------------------------------------------- PARAMETERS

uint2 PassViewSize;
float4 ScatteringViewportSize;
float ScatteringScaling;
uint MaxScatteringGroupCount;
float MinScatteringCocRadius;
float PreProcessingToProcessingCocRadiusFactor;
float BokehGatherDistinctionLimit;

float2 SensorSize;
float Aperture;
float LensImageDistance;
float BarrelRadius;
float BarrelLength;
uint4 MatteBoxPlanes[MAX_MATTE_BOX_FLAGS];

float4 GatherInputExtentAndInvExtent;
float4 GatherInputViewRectUV;
Texture2D GatherInput_SceneColor;
Texture2D GatherInput_SeparateCoc;

float4 QuarterResGatherInputExtentAndInvExtent;
float4 QuarterResGatherInputViewRectUV;
Texture2D QuarterResGatherInput_SceneColor;


//------------------------------------------------------- OUTPUTS

RWTexture2D<float4>	OutputMips_0_SceneColor;
RWTexture2D<float4>	OutputMips_1_SceneColor;
RWTexture2D<float4>	OutputMips_2_SceneColor;
RWTexture2D<float4>	OutputMips_3_SceneColor;

#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC || CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC

RWTexture2D<float>	OutputMips_0_SeparateCoc;
RWTexture2D<float>	OutputMips_1_SeparateCoc;
RWTexture2D<float>	OutputMips_2_SeparateCoc;
RWTexture2D<float>	OutputMips_3_SeparateCoc;

#endif

#if DIM_HYBRID_SCATTER_FGD || DIM_HYBRID_SCATTER_BGD
	RWBuffer<uint> OutScatterDrawIndirectParameters;

	RWStructuredBuffer<float4> OutForegroundScatterDrawList;
	RWStructuredBuffer<float4> OutBackgroundScatterDrawList;
#endif


//------------------------------------------------------- LDS

#if SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_ATOMIC
	groupshared uint SharedForegroundScatterGroupMask;
	groupshared uint SharedBackgroundScatterGroupMask;

#elif SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_LDS
	groupshared uint SharedForegroundScatterGroupMask[THREADGROUP_TOTALSIZE];
	groupshared uint SharedBackgroundScatterGroupMask[THREADGROUP_TOTALSIZE];

#endif

groupshared uint SharedForegroundAtomic;
groupshared uint SharedBackgroundAtomic;

groupshared float4 GroupSharedArray[THREADGROUP_TOTALSIZE];
#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC
	groupshared float GroupSharedArray2[THREADGROUP_TOTALSIZE];
#endif


//------------------------------------------------------- FUNCTIONS

struct FReductionInput
{
	float4 Colors[4];
	float CocRadii[4];
};

struct FReductionOutput
{
	float4 Color;
	float CocRadius;
};

FReductionInput ReadReductionInputFromLDS(uint OutId, uint ReduceBankSize)
{
	FReductionInput Result;

	// Gather the 4 input samples. Eaches samples are on banks of size <ReduceBankSize> for coherent LDS memory access.
	UNROLL
	for (uint i = 0; i < 4; i++)
	{
		uint InSharedId = OutId + i * ReduceBankSize;
		
		#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC
		{
			Result.Colors[i] = GroupSharedArray[InSharedId];
			Result.CocRadii[i] = GroupSharedArray2[InSharedId];
		}
		#else
		{
			Result.Colors[i] = GroupSharedArray[InSharedId];
			Result.CocRadii[i] = Result.Colors[i].a;
		}
		#endif
	}

	return Result;
}

void WriteReductionOutputToLDS(FReductionOutput Output, uint OutId)
{
	// Output to shared memory.
	#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC
	{
		GroupSharedArray[OutId] = Output.Color;
		GroupSharedArray2[OutId] = Output.CocRadius;
	}
	#else
	{
		GroupSharedArray[OutId] = float4(Output.Color.rgb, Output.CocRadius);
	}
	#endif
}

FReductionOutput ReduceOperator(FReductionInput Input, FCocDownsampleParams DownsampleParams, uint OutId)
{
	// Downsample the 4 sample to one according to Coc.
	float4 OutColor;
	float OutCocRadius;
	DownsampleSceneColorWithCoc(DownsampleParams, Input.Colors, Input.CocRadii, OutColor, OutCocRadius); 

	FReductionOutput Output;
	Output.Color = OutColor;
	Output.CocRadius = OutCocRadius;

	return Output;
}

// Output buffer.
void OutputMipLevel(const uint MipLevel, uint2 MipPixelPos, float4 Color, float CocRadius)
{
	#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_COC
		float4 OutputSceneColor = float4(Color.rgb, CocRadius);
	#elif CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC
		float4 OutputSceneColor = float4(Color.rgb, 0);
		float OutputSeparateCoc = CocRadius;
	#else
		float4 OutputSceneColor = Color;
		float OutputSeparateCoc = CocRadius;
	#endif
	
	// This is hugly, but compile time.
	#if CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_COC
	{
		if (MipLevel == 0)
		{
			OutputMips_0_SceneColor[MipPixelPos] = OutputSceneColor;
		}
#if DIM_REDUCE_MIP_COUNT > 1
		else if (MipLevel == 1)
		{
			OutputMips_1_SceneColor[MipPixelPos] = OutputSceneColor;
		}
#if DIM_REDUCE_MIP_COUNT > 2
		else if (MipLevel == 2)
		{
			OutputMips_2_SceneColor[MipPixelPos] = OutputSceneColor;
		}
#if DIM_REDUCE_MIP_COUNT > 3
		else if (MipLevel == 3)
		{
			OutputMips_3_SceneColor[MipPixelPos] = OutputSceneColor;
		}
#endif
#endif
#endif
	}
	#elif CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_ALPHA_COC || CONFIG_GATHER_INPUT_LAYOUT == GATHER_INPUT_LAYOUT_RGB_SEPARATE_COC
	{
		if (MipLevel == 0)
		{
			OutputMips_0_SceneColor[MipPixelPos] = OutputSceneColor;
			OutputMips_0_SeparateCoc[MipPixelPos] = OutputSeparateCoc;
		}
#if DIM_REDUCE_MIP_COUNT > 1
		else if (MipLevel == 1)
		{
			OutputMips_1_SceneColor[MipPixelPos] = OutputSceneColor;
			OutputMips_1_SeparateCoc[MipPixelPos] = OutputSeparateCoc;
		}
#if DIM_REDUCE_MIP_COUNT > 2
		else if (MipLevel == 2)
		{
			OutputMips_2_SceneColor[MipPixelPos] = OutputSceneColor;
			OutputMips_2_SeparateCoc[MipPixelPos] = OutputSeparateCoc;
		}
#if DIM_REDUCE_MIP_COUNT > 3
		else if (MipLevel == 3)
		{
			OutputMips_3_SceneColor[MipPixelPos] = OutputSceneColor;
			OutputMips_3_SeparateCoc[MipPixelPos] = OutputSeparateCoc;
		}
#endif
#endif
#endif
	}
	#else
		#error Unknown gather input layout.
	#endif
}


//------------------------------------------------------- ENTRY POINT

// Pixels that are bright and have a large CoC should be scattered instead of gathered, to avoid gather noise.
// If they aren't, e.g. due to performance reasons, we may prefer to cull them instead of gathering them anyway.
// This will bias the image, as bokeh may appear slightly darker, but helps clean up noise in large bokeh.
float ApplyBokehGatherDistinctionFilter(float CocRadius, float4 Color, float FrameExposureScale)
{
	if (BokehGatherDistinctionLimit != 0)
	{
		// see also ComputeLuminanceOnlyBasedScatterFactor
		float PerceivedLuma = Luma4(Color.rgb) * FrameExposureScale;
		float BokehDistinction = abs(CocRadius) * PerceivedLuma;
		return smoothstep(BokehGatherDistinctionLimit + 200, BokehGatherDistinctionLimit - 200, BokehDistinction);
	}
	else
	{
		return 1;
	}
}

void ComputeVignetteData(uint2 DispatchThreadId, float CocRadius, out float4 Output[VIGNETTE_DATA_PER_PIXEL])
{
	const float2 DrawPosition = (DispatchThreadId & 0xFFFFFFFE) + 0.5;
	const float2 BokehCenterScreenPos = ScatteringScaling * DrawPosition;
	const float2 BokehCenterClipPos = (BokehCenterScreenPos * ScatteringViewportSize.zw) * 2.0 - 1.0;
	const float3 ObjectDirection = normalize(-float3(BokehCenterClipPos * (SensorSize*0.5), -LensImageDistance));

	const float Focus = View.DepthOfFieldFocalDistance;
	const float2 Barrel = asfloat(ProjectAndPackCylinder(
		CocRadius, 
		BokehCenterClipPos, 
		BarrelRadius, 
		BarrelLength, 
		SensorSize, 
		Focus, 
		Aperture, 
		CocInfinityRadius, 
		LensImageDistance));

	float2 ProjectedMatteBoxFlags[MAX_MATTE_BOX_FLAGS];
	UNROLL
	for (int Index = 0; Index < MAX_MATTE_BOX_FLAGS; ++Index)
	{
		ProjectedMatteBoxFlags[Index] = asfloat(ProjectAndPackMatteBoxFlag(ObjectDirection, BarrelRadius, BarrelLength, MatteBoxPlanes[Index]));
	}

	Output[0] = float4(Barrel, ProjectedMatteBoxFlags[0]);
	Output[1] = float4(ProjectedMatteBoxFlags[1], ProjectedMatteBoxFlags[2]);
}

[numthreads(THREADGROUP_TILE_SIZE, THREADGROUP_TILE_SIZE, 1)]
void ReduceCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex) 
{
	ResolvedView = ResolveView();

	// Init LDS.
	#if (DIM_HYBRID_SCATTER_FGD || DIM_HYBRID_SCATTER_BGD) && SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_ATOMIC
	{
		SharedForegroundScatterGroupMask = 0;
		SharedBackgroundScatterGroupMask = 0;

		GroupMemoryBarrierWithGroupSync();
	}
	#endif

	// Gets group thread id to be fully LDS coherent.
	#if CONFIG_WAVE_BROADCAST_REDUCTION
		uint2 GroupThreadId = uint2(GroupThreadIndex % THREADGROUP_TILE_SIZE, GroupThreadIndex / THREADGROUP_TILE_SIZE);
		
		uint ScatteringGroupIndex = (
			((GroupThreadIndex & 0x06) >> 1) |
			((GroupThreadIndex & 0x30) >> 2));
		uint ScatteringGroupThreadIndex = (
			( GroupThreadIndex & 0x1) |
			((GroupThreadIndex & 0x8) >> 2));

	#else
		uint2 GroupThreadId = InitialTilePixelPositionForReduction2x2(SHARED_REDUCE_COUNT, GroupThreadIndex);

		uint ScatteringGroupIndex = GroupThreadIndex & 0xF;
		uint ScatteringGroupThreadIndex = GroupThreadIndex >> 4;

	#endif

	uint2 DispatchThreadId = THREADGROUP_TILE_SIZE * GroupId + GroupThreadId;
	
	float2 BufferUV;
	{
		float2 PixelPosition = DispatchThreadId + 0.5;
		BufferUV = GatherInputViewRectUV.xy + (PixelPosition * GatherInputExtentAndInvExtent.zw);
		BufferUV = clamp(BufferUV, GatherInputViewRectUV.xy, GatherInputViewRectUV.zw);
	}

	// Fetch scene color.
	float4 GatheredColor = 0;
	float CocRadius = 0;
	#if CONFIG_DOF_ALPHA
	{
		GatheredColor = GatherInput_SceneColor.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
		CocRadius = GatherInput_SeparateCoc.SampleLevel(GlobalPointClampedSampler, BufferUV, 0).r;

		// Convert from translucency to opacity.
		#if !DIM_DISABLE_OUTPUT_MIP0
			GatheredColor.a = 1.0 - GatheredColor.a;
		#endif
	}
	#else
	{
		GatheredColor = GatherInput_SceneColor.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
		CocRadius = GatheredColor.a;
	}
	#endif

	// Sample the frame exposure to SGPR.
	float FrameExposureScale = ToScalarMemory(EyeAdaptationLookup() * View.OneOverPreExposure);

	// Indirect dispatch 
	#if DIM_HYBRID_SCATTER_FGD || DIM_HYBRID_SCATTER_BGD
	{
		const uint IndirectParameterSize = SCATTERING_INDIRECT_PARAMETER_SIZE;
		
		// Convert Coc radius from preprocessing basis to processing basis.
		float ScatterCocRadius = CocRadius * PreProcessingToProcessingCocRadiusFactor;

		// Decides whether should hybrid scatter or not.
		// It is fine to do it for sample outside viewport UV because they are discarded when the bokeh
		// is getting close to viewport edge.
		float ScatterFactor;
		{
			float2 QuarterResBufferUV;
			{
				float2 FullResViewRectPixelSize = (GatherInputViewRectUV.zw - GatherInputViewRectUV.xy) * GatherInputExtentAndInvExtent.xy;
				float2 QuarterResViewRectPixelSize = (QuarterResGatherInputViewRectUV.zw - QuarterResGatherInputViewRectUV.xy) * QuarterResGatherInputExtentAndInvExtent.xy;
				float2 Rescale = QuarterResViewRectPixelSize / FullResViewRectPixelSize;

				float2 PixelPosition = (BufferUV - GatherInputViewRectUV.xy) * GatherInputExtentAndInvExtent.xy * Rescale;
				QuarterResBufferUV = QuarterResGatherInputViewRectUV.xy + (PixelPosition * QuarterResGatherInputExtentAndInvExtent.zw);
			}

			FHybridScatterInputs Parameters;
			Parameters.NeighborhoodComparisonBuffer = QuarterResGatherInput_SceneColor;
			Parameters.NeighborhoodComparisonBufferInvExtent = QuarterResGatherInputExtentAndInvExtent.zw;
			Parameters.NeighborhoodComparisonViewRectUV = QuarterResGatherInputViewRectUV;

			Parameters.FrameExposureScale = FrameExposureScale;

			Parameters.Color = GatheredColor;
			Parameters.CocRadius = ScatterCocRadius;
			Parameters.BufferUV = QuarterResBufferUV;

			ScatterFactor = ComputeLuminanceOnlyBasedScatterFactor(Parameters);

			// Do not scater if the sample becomes really small.
			if (1)
			{
				#if DIM_HYBRID_SCATTER_FGD && !DIM_HYBRID_SCATTER_BGD
					ScatterFactor *= saturate(-ScatterCocRadius - MinScatteringCocRadius);

				#elif !DIM_HYBRID_SCATTER_FGD && DIM_HYBRID_SCATTER_BGD
					ScatterFactor *= saturate(ScatterCocRadius - MinScatteringCocRadius);

				#else
					ScatterFactor *= saturate(abs(ScatterCocRadius) - MinScatteringCocRadius);

				#endif
			}

			// Do not scater if the Coc start to draw outside the viewport.
			if (1)
			{
				float2 SvPosition = (DispatchThreadId + 0.5);

				float ClosestBorderDistance = float(min(
					min(SvPosition.x, SvPosition.y),
					min(PassViewSize.x - SvPosition.x, PassViewSize.y - SvPosition.y)));
	
				ScatterFactor *= saturate(ClosestBorderDistance - abs(ScatterCocRadius));
			}

			// Compare this sample with neighbor if worth it. 
			BRANCH
			if (ScatterFactor > 0.5)
			{
				ScatterFactor *= ComputeNeighborBasedScatterFactor(Parameters);
			}
			
			// Sharpen to remove sprites that are contributing not that much.
			if (1)
			{
				const float Sharpen = 2;
				ScatterFactor = saturate(ScatterFactor * Sharpen + (1 - Sharpen));
			}
		}

		uint ScatteringGroupMask = 1u << ScatteringGroupIndex;
		
		const float ScatterFactorThreshold = 0.01;
		bool bScatterThisColor = ScatterFactor > ScatterFactorThreshold;
		bool bIsForeground = IsForeground(CocRadius);

		if (!bScatterThisColor)
		{
			GatheredColor *= ApplyBokehGatherDistinctionFilter(CocRadius, GatheredColor, View.OneOverPreExposure);
		}
			
		// Whether scattering group actually scatter or not.
		uint ForegroundScatterGroupMask = 0;
		uint BackgroundScatterGroupMask = 0;
		#if SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_WAVE
		{
			#if DIM_HYBRID_SCATTER_FGD
				ForegroundScatterGroupMask = WaveActiveBitOr((bScatterThisColor && bIsForeground) ? ScatteringGroupMask : 0);
			#endif
			
			#if DIM_HYBRID_SCATTER_BGD
				BackgroundScatterGroupMask = WaveActiveBitOr((bScatterThisColor && !bIsForeground) ? ScatteringGroupMask : 0);
			#endif
		}
		#elif SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_ATOMIC
		// Atomic or whether scattering group actually scatter or not.
		{
			BRANCH
			if (DIM_HYBRID_SCATTER_FGD && bIsForeground && bScatterThisColor)
			{
				uint Unused;
				InterlockedOr(SharedForegroundScatterGroupMask, ScatteringGroupMask, Unused);
			}
			
			BRANCH
			if (DIM_HYBRID_SCATTER_BGD && !bIsForeground && bScatterThisColor)
			{
				uint Unused;
				InterlockedOr(SharedBackgroundScatterGroupMask, ScatteringGroupMask, Unused);
			}
			
			GroupMemoryBarrierWithGroupSync();

			#if DIM_HYBRID_SCATTER_FGD
				ForegroundScatterGroupMask = SharedForegroundScatterGroupMask;
			#endif
			#if DIM_HYBRID_SCATTER_BGD
				BackgroundScatterGroupMask = SharedBackgroundScatterGroupMask;
			#endif
		}
		#elif SCATTER_ALLOC_METHOD == SCATTER_ALLOC_METHOD_LDS
		{
			#if DIM_HYBRID_SCATTER_FGD
				SharedForegroundScatterGroupMask[GroupThreadIndex] = (
					(bScatterThisColor && bIsForeground) ? ScatteringGroupMask : 0);
			#endif

			#if DIM_HYBRID_SCATTER_BGD
				SharedBackgroundScatterGroupMask[GroupThreadIndex] = (
					(bScatterThisColor && !bIsForeground) ? ScatteringGroupMask : 0);
			#endif

			GroupMemoryBarrierWithGroupSync();

			UNROLL
			for (uint i = 0; i < THREADGROUP_TOTALSIZE; i++)
			{
				#if DIM_HYBRID_SCATTER_FGD
					ForegroundScatterGroupMask |= SharedForegroundScatterGroupMask[i];
				#endif
				
				#if DIM_HYBRID_SCATTER_BGD
					BackgroundScatterGroupMask |= SharedBackgroundScatterGroupMask[i];
				#endif
			}
		}
		#else
			#error Unknown scatter group allocation method.
		#endif // !COMPILER_SUPPORTS_WAVE_BIT_ORAND

		// Allocate globally.
		BRANCH
		if (GroupThreadIndex == 0)
		{
			BRANCH
			if (DIM_HYBRID_SCATTER_FGD && ForegroundScatterGroupMask)
			{
				uint ScatteringGroupCount = countbits(ForegroundScatterGroupMask);
				InterlockedAdd(OutScatterDrawIndirectParameters[1 + 0 * IndirectParameterSize], ScatteringGroupCount, SharedForegroundAtomic);
			}
		
			BRANCH
			if (DIM_HYBRID_SCATTER_BGD && BackgroundScatterGroupMask)
			{
				uint ScatteringGroupCount = countbits(BackgroundScatterGroupMask);
				InterlockedAdd(OutScatterDrawIndirectParameters[1 + 1 * IndirectParameterSize], ScatteringGroupCount, SharedBackgroundAtomic);
			}
		}
		
		#if COMPILER_SUPPORTS_WAVE_ONCE
			if (WaveGetLaneCount() < THREADGROUP_TILE_SIZE * THREADGROUP_TILE_SIZE)
			{
				GroupMemoryBarrierWithGroupSync();
			}
		#else
			GroupMemoryBarrierWithGroupSync();
		#endif
		
		// Intensity loss factor caused by the area of confusion.
		// TODO: no need min()
		float IntensityLossFactor = min(1, SafeRcp(PI * CocRadius * CocRadius, 1.0) * CocSqueeze);

		
		

		// Enqueue scattering group in foreground scatter draw list.
		BRANCH
		if (DIM_HYBRID_SCATTER_FGD && (ScatteringGroupMask & ForegroundScatterGroupMask))
		{
			uint GroupScatteringOffset = countbits(ForegroundScatterGroupMask & (ScatteringGroupMask - 1));
			uint IndirectDrawIndex = SharedForegroundAtomic + GroupScatteringOffset;
			
			BRANCH
			if (IndirectDrawIndex < MaxScatteringGroupCount)
			{
				BRANCH
				if (ScatteringGroupThreadIndex == 0)
				{
					OutForegroundScatterDrawList[SCATTER_DRAW_LIST_GROUP_STRIDE * IndirectDrawIndex + 0] = float4(DispatchThreadId + 0.5, 0.0, 0.0);
				}
				
				float4 ScatterColor = GatheredColor * (ScatterFactor * IntensityLossFactor * (IsForeground(ScatterCocRadius) ? 1 : 0));
				uint Offset = SCATTER_DRAW_LIST_GROUP_STRIDE * IndirectDrawIndex + 1 + (ScatteringGroupThreadIndex * SCATTER_DRAW_LIST_DATA_PER_PIXEL);
				OutForegroundScatterDrawList[Offset] = float4(ScatterColor.rgb, abs(ScatterCocRadius));

				if (BarrelLength >= 0)
				{
					float4 VignetteData[VIGNETTE_DATA_PER_PIXEL];
					ComputeVignetteData(DispatchThreadId, ScatterCocRadius, VignetteData);

					UNROLL
					for (uint Index = 0; Index < VIGNETTE_DATA_PER_PIXEL; ++Index)
					{
						OutForegroundScatterDrawList[Offset + 1 + Index] = VignetteData[Index];
					}
				}
			}
			else
			{
				// If not scatering this Scattering group index, then ensure ScatterFactor = 1 in case of the OutScatterDrawList was full.
				ScatterFactor = 0;
			}
		}

		// Enqueue scattering group in background scatter draw list.
		BRANCH
		if (DIM_HYBRID_SCATTER_BGD && (ScatteringGroupMask & BackgroundScatterGroupMask))
		{
			uint GroupScatteringOffset = countbits(BackgroundScatterGroupMask & (ScatteringGroupMask - 1));
			uint IndirectDrawIndex = SharedBackgroundAtomic + GroupScatteringOffset;
			
			BRANCH
			if (IndirectDrawIndex < MaxScatteringGroupCount)
			{
				BRANCH
				if (ScatteringGroupThreadIndex == 0)
				{
					OutBackgroundScatterDrawList[SCATTER_DRAW_LIST_GROUP_STRIDE * IndirectDrawIndex + 0] = float4(DispatchThreadId + 0.5, 0.0, 0.0);
				}
				
				float4 ScatterColor = GatheredColor * (ScatterFactor * IntensityLossFactor * (IsForeground(ScatterCocRadius) ? 0 : 1));
				uint Offset = SCATTER_DRAW_LIST_GROUP_STRIDE * IndirectDrawIndex + 1 + (ScatteringGroupThreadIndex * SCATTER_DRAW_LIST_DATA_PER_PIXEL);
				OutBackgroundScatterDrawList[Offset] = float4(ScatterColor.rgb, abs(ScatterCocRadius));

				if (BarrelLength >= 0)
				{
					float4 VignetteData[VIGNETTE_DATA_PER_PIXEL];
					ComputeVignetteData(DispatchThreadId, ScatterCocRadius, VignetteData);

					UNROLL
					for (uint Index = 0; Index < VIGNETTE_DATA_PER_PIXEL; ++Index)
					{
						OutBackgroundScatterDrawList[Offset + 1 + Index] = VignetteData[Index];
					}
				}
			}
			else
			{
				// If not scatering this Scattering group index, then ensure ScatterFactor = 1 in case of the OutScatterDrawList was full.
				ScatterFactor = 0;
			}
		}

		// Remove color intensity that has been scattered.
		GatheredColor.rgb *= (1 - ScatterFactor);
	}
	#endif // DIM_HYBRID_SCATTER_FGD || DIM_HYBRID_SCATTER_BGD
	
	// for gathering pass to avoid having to clamp UV in gathering pass. 
	uint2 OutputPixelId = DispatchThreadId;
	
	// Output mip 0 in output buffer that is always a multiple of group size. Unless reducing lower mips.
	#if !DIM_DISABLE_OUTPUT_MIP0
	{
		OutputMipLevel(/* MipLevel = */ 0, OutputPixelId, GatheredColor, CocRadius);
	}
	#endif

	FCocDownsampleParams DownsampleParams;
	DownsampleParams.CocRadiusMultiplier = 1.0;
	DownsampleParams.FrameExposureScale = FrameExposureScale;
	DownsampleParams.bDoColorBasedWeighting = false;

	// Store initial values to LDS to prepare for first reduction round
	{
		FReductionOutput ReductionInit;
		ReductionInit.Color = GatheredColor;
		ReductionInit.CocRadius = CocRadius;
		WriteReductionOutputToLDS(ReductionInit, GroupThreadIndex);
	}

	// Output mip > 0 of the gathering pass.
	UNROLL
	for (int MipLevel = 1; MipLevel < DIM_REDUCE_MIP_COUNT; MipLevel++)
	{
		// Multiplier that needs to be applied on Coc computation to be resolution independent.
		const uint TileSize = THREADGROUP_TILE_SIZE / (1u << MipLevel);
		const uint ReduceBankSize = TileSize * TileSize;

		DownsampleParams.CocRadiusMultiplier = 0.5 / float(1u << (MipLevel - 1));
		
		GroupMemoryBarrierWithGroupSync();

		FReductionOutput ReductionOutput = (FReductionOutput)0;
		if (GroupThreadIndex < ReduceBankSize)
		{
			FReductionInput ReductionInput = ReadReductionInputFromLDS(GroupThreadIndex, ReduceBankSize);
			ReductionOutput = ReduceOperator(ReductionInput, DownsampleParams, GroupThreadIndex);
		}
			
		GroupMemoryBarrierWithGroupSync();

		if (GroupThreadIndex < ReduceBankSize)
		{
			WriteReductionOutputToLDS(ReductionOutput, GroupThreadIndex);

			// Output mip map.
			OutputPixelId = OutputPixelId >> 1;
			OutputMipLevel(MipLevel, OutputPixelId, ReductionOutput.Color, ReductionOutput.CocRadius);
		}
	}
}
