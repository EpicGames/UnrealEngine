// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFSetup.usf: Diaphragm DOF's setup shader. 
=============================================================================*/

#define CONFIG_SETUP 1

#include "DOFDownsample.ush"


//------------------------------------------------------- COMPILE TIME CONSTANTS

#define THREADGROUP_SIZE_X (8)
#define THREADGROUP_SIZE_Y (8) 
#define THREADGROUP_SIZE_TOTAL (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)


//------------------------------------------------------- PARAMETERS

Texture2D SceneColorTexture;
Texture2D SceneDepthTexture;


//------------------------------------------------------- OUTPUTS

#if CONFIG_DOF_ALPHA
RWTexture2D<float>	Output0;
RWTexture2D<float4>	Output1;
RWTexture2D<float>	Output2;

#else
RWTexture2D<float4>	Output0;
RWTexture2D<float4>	Output1;

#endif

//------------------------------------------------------- FUNCTIONS


struct FDOFSetupParameters
{
	// Compile-time variables
	bool bOutputFullRes;
	bool bOutputHalfRes;

	Texture2D SceneColorTexture;
	Texture2D SceneDepthTexture;
};

struct FDOFSetupOutput
{
	// Requires bOutputFullRes
	bool bFullResIsValid;
	uint2 FullResOutputPixelPosition;
	float4 FullResColor;
	float FullResCocRadius;

	// Requires bOutputHalfRes
	bool bHalfResIsValid;
	uint2 HalfResOutputPixelPosition;
	float4 HalfResColor;
	float HalfResCocRadius;
};

// Calculate a 2D space-filling curve, similar to Morton, from a 1D coordinate.
// From the AMD GPUOpen "Optimizing for the Radeon RDNA architecture"
//   00 01 08 09 16 17 24 25
//   02 03 10 11 18 19 26 27
//   04 05 12 13 20 21 28 29
//   06 07 14 15 22 23 30 31
//   32 33 40 41 48 49 56 57
//   34 35 42 43 50 51 58 59
//   36 37 44 45 52 53 60 61
//   38 39 46 47 54 55 62 63
uint2 MortonLikeDecode(uint Index)
{
	uint2 Result;
	Result.x = (((Index >> 2) & 0x0007) & 0xFFFE) | (Index & 0x0001);
	Result.y = ((Index >> 1) & 0x0003) | (((Index >> 3) & 0x0007) & 0xFFFC);
	return Result;
}

#if WAVE_OPS
groupshared uint GroupNumWaves;

struct FSoftwareQuad
{
	uint WaveQuadLeaderLane;
	bool bIsLeaderLane;
	uint2 LocalPixelPosition;

	float4 LocalColor;
	float LocalCocRadius;

	void Setup(uint2 GroupId, uint GroupThreadIndex)
	{
		uint GroupWaveIndex = 0;
		if (WaveGetLaneCount() < THREADGROUP_SIZE_TOTAL)
		{
			if(GroupThreadIndex == 0)
			{
				GroupNumWaves = 0;
			}
			GroupMemoryBarrierWithGroupSync();

			if(WaveIsFirstLane())
			{
				InterlockedAdd(GroupNumWaves, 1, GroupWaveIndex);
			}
			GroupWaveIndex = WaveReadLaneFirst(GroupWaveIndex);
		}

		uint2 GroupSize = uint2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y);

		// Index of lane in group
		uint GroupLane = (GroupWaveIndex * WaveGetLaneCount()) + WaveGetLaneIndex();

		// Map threads to pixels using morton-like curve for better texture cache hit rate
		LocalPixelPosition = (GroupSize * GroupId) + MortonLikeDecode(GroupLane);
		WaveQuadLeaderLane = WaveGetLaneIndex() & (~(4 - 1));
		bIsLeaderLane = (WaveGetLaneIndex() & (4 - 1)) == 0;
	}

	bool IsLeader()
	{
		return bIsLeaderLane;
	}

	uint2 GetLocalPixelPosition()
	{
		return LocalPixelPosition;
	}

	void SetLocalPixel(float4 Color, float CocRadius)
	{
		LocalColor = Color;
		LocalCocRadius = CocRadius;
	}

	void Sync()
	{ }

	void GetPixel(uint PixelIndex, out float4 Color, out float CocRadius)
	{
		// Could maybe also use QuadReadLaneAt where it is supported

		#if COMPILER_SUPPORTS_WAVE_SWIZZLE_GCN
			const uint AndMask = ~(4 - 1);
			const uint OrMask = PixelIndex;
			Color.r = asfloat(WaveLaneSwizzleGCN(asuint(LocalColor.r), AndMask, OrMask, 0));
			Color.g = asfloat(WaveLaneSwizzleGCN(asuint(LocalColor.g), AndMask, OrMask, 0));
			Color.b = asfloat(WaveLaneSwizzleGCN(asuint(LocalColor.b), AndMask, OrMask, 0));
			Color.a = asfloat(WaveLaneSwizzleGCN(asuint(LocalColor.a), AndMask, OrMask, 0));
			CocRadius = asfloat(WaveLaneSwizzleGCN(asuint(LocalCocRadius), AndMask, OrMask, 0));
		#elif COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA
			Color.r = asfloat(WaveLaneRotateSwizzleRDNA(asuint(LocalColor.r), PixelIndex, 0));
			Color.g = asfloat(WaveLaneRotateSwizzleRDNA(asuint(LocalColor.g), PixelIndex, 0));
			Color.b = asfloat(WaveLaneRotateSwizzleRDNA(asuint(LocalColor.b), PixelIndex, 0));
			Color.a = asfloat(WaveLaneRotateSwizzleRDNA(asuint(LocalColor.a), PixelIndex, 0));
			CocRadius = asfloat(WaveLaneRotateSwizzleRDNA(asuint(LocalCocRadius), PixelIndex, 0));
		#else
			Color = WaveReadLaneAt(LocalColor, WaveQuadLeaderLane + PixelIndex);
			CocRadius = WaveReadLaneAt(LocalCocRadius, WaveQuadLeaderLane + PixelIndex);
		#endif // COMPILER_SUPPORTS_WAVE_SWIZZLE_RDNA
	}
};

#else // !WAVE_OPS

groupshared float4 SoftQuadColors[THREADGROUP_SIZE_TOTAL];
groupshared float SoftQuadCocRadius[THREADGROUP_SIZE_TOTAL];
struct FSoftwareQuad
{
	uint GroupLaneIndex;
	uint QuadLeaderLane;
	bool bIsLeaderLane;
	uint2 LocalPixelPosition;

	void Setup(uint2 GroupId, uint GroupThreadIndex)
	{
		uint2 GroupSize = uint2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y);

		// Map threads to pixels using morton-like curve for better texture cache hit rate
		GroupLaneIndex = GroupThreadIndex;
		LocalPixelPosition = (GroupSize * GroupId) + MortonLikeDecode(GroupThreadIndex);
		QuadLeaderLane = GroupThreadIndex & (~(4 - 1));
		bIsLeaderLane = (GroupThreadIndex & (4 - 1)) == 0;
	}

	bool IsLeader()
	{
		return bIsLeaderLane;
	}

	uint2 GetLocalPixelPosition()
	{
		return LocalPixelPosition;
	}

	void SetLocalPixel(float4 Color, float CocRadius)
	{
		SoftQuadColors[GroupLaneIndex] = Color;
		SoftQuadCocRadius[GroupLaneIndex] = CocRadius;
	}

	void Sync()
	{
		GroupMemoryBarrierWithGroupSync();
	}

	void GetPixel(uint PixelIndex, out float4 Color, out float CocRadius)
	{
		Color = 0;
		CocRadius = 0;
		
		if (IsLeader())
		{
			Color = SoftQuadColors[QuadLeaderLane+PixelIndex];
			CocRadius = SoftQuadCocRadius[QuadLeaderLane+PixelIndex];
		}
	}
};

#endif // WAVE_OPS

// Launch with 1 thread per fullres pixel, and 8x8 groupsize
FDOFSetupOutput SetupDOF(
	FDOFSetupParameters Parameters,
	uint2 GroupId,
	uint GroupThreadIndex,
	uint2 DispatchThreadId
)
{
	FDOFSetupOutput Output = (FDOFSetupOutput)0;

	if(Parameters.bOutputFullRes && Parameters.bOutputHalfRes)
	{
		uint2 GroupSize = uint2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y);
		if(any((GroupId * GroupSize) >= View.ViewRectMinAndSize.zw))
		{
			return Output;
		}
		
		// group is twice as large in x and y as in halfres/fullres case
		// group is subdivided in 4 subgroups of 8x8 (2x2 subgroups)
		// 1 thread is 1 fullres pixel
		// 1 thread from first subgroup is 1 halfres pixel

		FSoftwareQuad Quad;
		Quad.Setup(GroupId, GroupThreadIndex);

		uint2 InputPixelPosition = uint2(View.ViewRectMin.xy) + min(Quad.GetLocalPixelPosition(), View.ViewRectMinAndSize.zw);
		Output.FullResOutputPixelPosition = uint2(View.ViewRectMin.xy) + Quad.GetLocalPixelPosition();
		
		Output.bFullResIsValid = all(Quad.GetLocalPixelPosition() < View.ViewRectMinAndSize.zw);

		// Fetch scene color/depth, and compute CocRadius.
		float2 SceneBufferUV = (InputPixelPosition + 0.5) * View.BufferSizeAndInvSize.zw;
	
		if (true)
		{
			SceneBufferUV = clamp(SceneBufferUV, View.BufferBilinearUVMinMax.xy, View.BufferBilinearUVMinMax.zw);
		}

		float4 SceneColor = Parameters.SceneColorTexture.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0);
		float DeviceZ = Parameters.SceneDepthTexture.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0).r;
		float CocRadius = SceneDepthToCocRadius(ConvertFromDeviceZ(DeviceZ));
	
		// Output full resolution.
		Output.FullResColor = SceneColor;
		Output.FullResCocRadius = CocRadius;

		Quad.SetLocalPixel(SceneColor, CocRadius);
		Quad.Sync();

		float4 SceneColors[4];
		float CocRadii[4];

		UNROLL
		for (uint Index = 0; Index < 4; Index++)
		{
			Quad.GetPixel(Index, /* out */ SceneColors[Index], /* out */ CocRadii[Index]);
		}

		BRANCH
		if (Quad.IsLeader())
		{
			Output.bHalfResIsValid = true;
			Output.HalfResOutputPixelPosition = InputPixelPosition / 2;

			FCocDownsampleParams DownsampleParams;
			DownsampleParams.CocRadiusMultiplier = 1.0;
			DownsampleParams.FrameExposureScale = 1.0;
			DownsampleParams.bDoColorBasedWeighting = false;

			DownsampleSceneColorWithCoc(DownsampleParams, SceneColors, CocRadii, Output.HalfResColor, Output.HalfResCocRadius);
		}
	}
	else if(Parameters.bOutputFullRes && !Parameters.bOutputHalfRes)
	{
		if(any(DispatchThreadId >= View.ViewRectMinAndSize.zw))
		{
			return Output;
		}

		Output.bFullResIsValid = true;

		uint2 InputPixelPosition = View.ViewRectMin.xy + DispatchThreadId;

		Output.FullResOutputPixelPosition = InputPixelPosition;
		float2 SceneBufferUV = (InputPixelPosition + 0.5) * View.BufferSizeAndInvSize.zw;
	
		Output.FullResColor = Parameters.SceneColorTexture.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0);
	
		float DeviceZ = Parameters.SceneDepthTexture.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0).r;
		Output.FullResCocRadius = SceneDepthToCocRadius(ConvertFromDeviceZ(DeviceZ));
	}
	else if(!Parameters.bOutputFullRes && Parameters.bOutputHalfRes)
	{
		uint2 HalfResViewSize = uint2(
			DivideAndRoundUp(View.ViewRectMinAndSize.z, 2), 
			DivideAndRoundUp(View.ViewRectMinAndSize.w, 2)
		);
		if(any(DispatchThreadId >= HalfResViewSize))
		{
			return Output;
		}

		Output.bHalfResIsValid = true;
		
		Output.HalfResOutputPixelPosition = (View.ViewRectMin.xy / 2) + DispatchThreadId;
		float2 SceneBufferUV = (float2(View.ViewRectMin.xy) + 2.0 * (DispatchThreadId + 0.5)) * View.BufferSizeAndInvSize.zw;
		float2 SceneBufferUVOffset = View.BufferSizeAndInvSize.zw * 0.5;
			
		FCocDownsampleParams DownsampleParams;
		DownsampleParams.CocRadiusMultiplier = 1.0;
		DownsampleParams.FrameExposureScale = 1.0;
		DownsampleParams.bDoColorBasedWeighting = false;

		uint2 GroupSize = uint2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y);
		uint2 ViewportSize = uint2(
			DivideAndRoundUp(View.ViewRectMinAndSize.z, 2),
			DivideAndRoundUp(View.ViewRectMinAndSize.w, 2));
		bool bIsBorderPixel = any(GroupId * GroupSize + GroupSize >= ViewportSize); 

		float CocRadii[4];
		float4 SceneColors[4];

		BRANCH
		if (!bIsBorderPixel)
		{
			// Fetch Coc
			float4 SceneDepthGather = Parameters.SceneDepthTexture.Gather(GlobalPointClampedSampler, SceneBufferUV);
			CocRadii[0] = SceneDepthToCocRadius(ConvertFromDeviceZ(SceneDepthGather.w));
			CocRadii[1] = SceneDepthToCocRadius(ConvertFromDeviceZ(SceneDepthGather.z));
			CocRadii[2] = SceneDepthToCocRadius(ConvertFromDeviceZ(SceneDepthGather.x));
			CocRadii[3] = SceneDepthToCocRadius(ConvertFromDeviceZ(SceneDepthGather.y));

			// Fetch color.
			for (uint i = 0; i < 4; i++)
			{
				float2 SampleUV = SceneBufferUV + SceneBufferUVOffset * float2(kOffsetsCross3x3[i]);
				SceneColors[i] = Parameters.SceneColorTexture.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);
			}
		}
		else
		{
			// Sample scene color and depth.
			
			for (uint i = 0; i < 4; i++)
			{
				float2 SampleUV = SceneBufferUV + SceneBufferUVOffset * float2(kOffsetsCross3x3[i]);

				if (true)
				{
					SampleUV = clamp(SampleUV, View.BufferBilinearUVMinMax.xy, View.BufferBilinearUVMinMax.zw);
				}

				SceneColors[i] = Parameters.SceneColorTexture.SampleLevel(GlobalPointClampedSampler, SampleUV, 0);
			
				float SampleDeviceZ = Parameters.SceneDepthTexture.SampleLevel(GlobalPointClampedSampler, SampleUV, 0).r;
				CocRadii[i] = SceneDepthToCocRadius(ConvertFromDeviceZ(SampleDeviceZ));
			}
		}

		DownsampleSceneColorWithCoc(DownsampleParams, SceneColors, CocRadii, Output.HalfResColor, Output.HalfResCocRadius);
	}

	return Output;
}


#if DIM_OUTPUT_RES_DIVISOR == 0 // Output full & half resolution.

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void SetupCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex,
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	FDOFSetupParameters Parameters;
	Parameters.bOutputFullRes = true;
	Parameters.bOutputHalfRes = true;
	
	Parameters.SceneColorTexture = SceneColorTexture;
	Parameters.SceneDepthTexture = SceneDepthTexture;

	FDOFSetupOutput Output = SetupDOF(Parameters, GroupId, GroupThreadIndex, DispatchThreadId);

	if (Output.bFullResIsValid)
	{
		#if CONFIG_DOF_ALPHA
		{
			Output0[Output.FullResOutputPixelPosition] = Output.FullResCocRadius;
		}
		#else
		{
			Output0[Output.FullResOutputPixelPosition] = float4(Output.FullResColor.rgb, Output.FullResCocRadius);
		}
		#endif
	}

	if (Output.bHalfResIsValid)
	{
		#if CONFIG_DOF_ALPHA
		{
			Output1[Output.HalfResOutputPixelPosition] = Output.HalfResColor;
			Output2[Output.HalfResOutputPixelPosition] = Output.HalfResCocRadius;
		}
		#else
		{
			Output1[Output.HalfResOutputPixelPosition] = float4(Output.HalfResColor.rgb, Output.HalfResCocRadius);
		}
		#endif
	}
}


#elif DIM_OUTPUT_RES_DIVISOR == 1 // Output full resolution only.

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void SetupCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex,
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	FDOFSetupParameters Parameters;
	Parameters.bOutputFullRes = true;
	Parameters.bOutputHalfRes = false;
	
	Parameters.SceneColorTexture = SceneColorTexture;
	Parameters.SceneDepthTexture = SceneDepthTexture;

	FDOFSetupOutput Output = SetupDOF(Parameters, GroupId, GroupThreadIndex, DispatchThreadId);

	if (Output.bFullResIsValid)
	{
		#if CONFIG_DOF_ALPHA
		{
			Output0[Output.FullResOutputPixelPosition] = Output.FullResCocRadius;
		}
		#else
		{
			Output0[Output.FullResOutputPixelPosition] = float4(Output.FullResColor.rgb, Output.FullResCocRadius);
		}
		#endif
	}
}


#elif DIM_OUTPUT_RES_DIVISOR == 2 // Output half resolution only.

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void SetupCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex,
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	FDOFSetupParameters Parameters;
	Parameters.bOutputFullRes = false;
	Parameters.bOutputHalfRes = true;
	
	Parameters.SceneColorTexture = SceneColorTexture;
	Parameters.SceneDepthTexture = SceneDepthTexture;

	FDOFSetupOutput Output = SetupDOF(Parameters, GroupId, GroupThreadIndex, DispatchThreadId);

	if (Output.bHalfResIsValid)
	{
		#if CONFIG_DOF_ALPHA
		{
			Output1[Output.HalfResOutputPixelPosition] = Output.HalfResColor;
			Output2[Output.HalfResOutputPixelPosition] = Output.HalfResCocRadius;
		}
		#else
		{
			Output1[Output.HalfResOutputPixelPosition] = float4(Output.HalfResColor.rgb, Output.HalfResCocRadius);
		}
		#endif
	}
}


#else
	#error Unkown resolution divisor.
#endif
