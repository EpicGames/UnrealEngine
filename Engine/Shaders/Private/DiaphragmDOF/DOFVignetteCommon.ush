// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOF/DOFVignetteCommon.ush: Various functions to calculate intersections
	between DOF rays and close occluders such as the lens barrel or matte box.
=============================================================================*/

#include "/Engine/Public/DualPixelVectorization.ush"

//------------------------------------------------------- BARREL

/// Given a ray and a cylinder (barrel) around the origin, calculate the height on the barrel where the ray hits.
float HitTestCylinder(float2 RayOrigin, float3 RayDirection, float BarrelRadius, float BarrelLength)
{
	const float2 P = RayOrigin;
	const float3 D = RayDirection;

	const float A = dot(D.xy, D.xy);
	const float B = 2.0 * dot(D.xy, P.xy);
	const float C = dot(P.xy, P.xy) - BarrelRadius*BarrelRadius;
	const float Det = B * B - 4 * A * C;
	const float SqrtDet = sqrt(Det);
	const float Inv2A = rcp(2 * A);
	const float T1 = (-B + SqrtDet) * Inv2A;
	const float T2 = (-B - SqrtDet) * Inv2A;
	const float L1 = T1 * D.z;
	const float L2 = T2 * D.z;
	const float L = min(T1 > 0 ? L1 : 1e20, T2 > 0 ? L2 : 1e20);
	const float HitOnBarrel = BarrelLength - L;
	return HitOnBarrel;
}

/// Project the aperture onto the end of a cylinder (lens barrel).
/// Use the output for fast vignette occlusion testing.
uint2 ProjectAndPackCylinder(
	const float CocRadius,
	const float2 BokehCenterClipPos,
	const float BarrelRadius,
	const float BarrelLength,
	const float2 SensorSize,
	const float Focus,
	const float Aperture,
	const float CocInfinityRadius,
	const float LensImageDistance
)
{
	const float SampleDepth = Focus / (1.0 - (CocRadius / CocInfinityRadius));
	const float CocRadiusAtBarrelEnd = lerp(Aperture/2, 0, BarrelRadius/SampleDepth);
	const float Factor = -1.0 * CocRadiusAtBarrelEnd / abs(CocRadius);

	const float2 CocCenterAtBarrelEnd = BokehCenterClipPos * (SensorSize * 0.5) * (BarrelLength / LensImageDistance) * -1;

	uint Packed0 = PackFloat2ToUInt(CocCenterAtBarrelEnd);
	uint Packed1 = asuint(Factor);
	uint2 Packed = uint2(Packed0, Packed1);

	return Packed;
}

float HitTestProjectedCylinder(
	const float2 PixelOffset, 
	const float2 BarrelData, 
	const float BarrelRadius,
	out float2 SamplePositionAtBarrelEnd
)
{
	const float2 CocCenterAtBarrelEnd = UnpackFloat2FromUInt(asuint(BarrelData.x));
	const float Factor = BarrelData.y;

	SamplePositionAtBarrelEnd = CocCenterAtBarrelEnd + /*sign(CocRadius) */ PixelOffset * Factor;
	return dot(SamplePositionAtBarrelEnd, SamplePositionAtBarrelEnd) < BarrelRadius*BarrelRadius;
}

half2 HitTestProjectedCylinder_DPV(
	const half2x2 PixelOffset, 
	const uint2 BarrelData[2], 
	const float BarrelRadius,
	out half2x2 SamplePositionAtBarrelEnd
)
{
	const half2x2 CocCenterAtBarrelEnd = dpv_interleave_registers(
		half2(UnpackFloat2FromUInt(BarrelData[0].x)), 
		half2(UnpackFloat2FromUInt(BarrelData[1].x))
	);
	const half2 Factor = dpv_interleave_registers(half(asfloat(BarrelData[0].y)), half(asfloat(BarrelData[1].y)));
	
	SamplePositionAtBarrelEnd = dpv_add(CocCenterAtBarrelEnd, dpv_mul(PixelOffset, Factor));
	const half BarrelRadiusSqr = half(BarrelRadius*BarrelRadius);
	return saturate(BarrelRadiusSqr - dpv_dot(SamplePositionAtBarrelEnd, SamplePositionAtBarrelEnd));
}

//------------------------------------------------------- MATTE BOX

/// Test a ray for intersection with a single panel on a matte box at the end of the lens barrel.
float HitTestMatteBoxFlag(float3 RayOrigin, float3 RayDirection, float BarrelRadius, float BarrelLength, uint4 FlagDefinition)
{
	float2 Unpacked0 = UnpackFloat2FromUInt(FlagDefinition.x);
	float2 Unpacked1 = UnpackFloat2FromUInt(FlagDefinition.y);

	float SinPlanePitch = Unpacked0.x;
	float CosPlanePitch = Unpacked0.y;
	float SinPlaneRoll = Unpacked1.x;
	float CosPlaneRoll = Unpacked1.y;

	float RcpSinPlanePitch = asfloat(FlagDefinition.z);
	float PlaneLength = asfloat(FlagDefinition.w);

	if (PlaneLength > 0)
	{
		float3 PlaneNormal = float3(CosPlaneRoll*SinPlanePitch, SinPlaneRoll*SinPlanePitch, CosPlanePitch);
		float PlaneDist = SinPlanePitch*BarrelRadius + CosPlanePitch*BarrelLength;

		float HitAngle = dot(RayDirection, PlaneNormal);
		if (HitAngle != 0)
		{
			float Hit = (-dot(PlaneNormal, RayOrigin) + PlaneDist) / HitAngle;
			if (Hit > 0)
			{
				float3 HitPoint = RayOrigin + Hit * RayDirection;

				bool bIsHit = BarrelLength < HitPoint.z && HitPoint.z < (BarrelLength + SinPlanePitch*PlaneLength);
				return !bIsHit;
			}
		}
	}

	return 1;
}

/// Project a matte box flag for onto the end of the lens barrel along a given object direction.
/// Use the output for fast vignette occlusion testing.
uint2 ProjectAndPackMatteBoxFlag(
	const float3 ObjectDirection,
	const float BarrelRadius,
	const float BarrelLength,
	const uint4 FlagDefinition
)
{
	float2 Unpacked0 = UnpackFloat2FromUInt(FlagDefinition.x);
	float2 Unpacked1 = UnpackFloat2FromUInt(FlagDefinition.y);

	float SinPitch = Unpacked0.x;
	float CosPitch = Unpacked0.y;
	float SinRoll = Unpacked1.x;
	float CosRoll = Unpacked1.y;

	float RcpSinPlanePitch = asfloat(FlagDefinition.z);
	float FlagLength = asfloat(FlagDefinition.w);

	// Find point on the end of the flag
	float3 FlagPoint = float3(
		CosRoll * (BarrelRadius - CosPitch * FlagLength),
		SinRoll * (BarrelRadius - CosPitch * FlagLength),
		BarrelLength + FlagLength * SinPitch);

	// Project point onto the barrel end
	float Step = (FlagLength * SinPitch) / ObjectDirection.z;
	float2 BarrelPoint = (FlagPoint - ObjectDirection * Step).xy;

	// Find projected flag edge direction
	float3 FlagPlaneNormal = float3(
		CosRoll * SinPitch,
		SinRoll * SinPitch,
		CosPitch);
	float2 EdgeDirection = float2(FlagPlaneNormal.y, -FlagPlaneNormal.x); // cross(FlagPlaneNormal, (0,0,1))

	uint Packed0 = PackFloat2ToUInt(BarrelPoint);
	uint Packed1 = PackFloat2ToUInt(EdgeDirection);
	uint2 Packed = uint2(Packed0, Packed1);
	
	return Packed;
}

float HitTestProjectedMatteBoxFlag(
	const float2 SamplePositionAtBarrelEnd, 
	const uint2 PackedProjectedMatteBoxFlag
)
{
	float2 BarrelPoint   = UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag.x);
	float2 EdgeDirection = UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag.y);

	// Check if sample point is on the left or right side of the line (BarrelPoint + t * EdgeDirection)
	float2 Delta = SamplePositionAtBarrelEnd - BarrelPoint;
	float Angle = Delta.x * EdgeDirection.y - Delta.y * EdgeDirection.x;

	return Angle > 0;
}

half2 HitTestProjectedMatteBoxFlag_DPV(
	const half2x2 SamplePositionAtBarrelEnd, 
	const uint2 PackedProjectedMatteBoxFlag[2]
)
{
	half2x2 BarrelPoint = dpv_interleave_registers(
		(half2)UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag[0].x),
		(half2)UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag[1].x)
	);
	half2x2 EdgeDirection = dpv_interleave_registers(
		(half2)UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag[0].y),
		(half2)UnpackFloat2FromUInt(PackedProjectedMatteBoxFlag[1].y)
	);

	// Check if sample point is on the left or right side of the line (BarrelPoint + t * EdgeDirection)
	half2x2 Delta = dpv_sub(SamplePositionAtBarrelEnd, BarrelPoint);
	half2 Angle = dpv_cross(Delta, EdgeDirection);

	const half Sharpness = 128.0;
	return saturate(Angle * Sharpness); // Angle > 0
}
