// Copyright Epic Games, Inc. All Rights Reserved.

//------------------------------------------------------- ENUM VALUES

#include "ScreenSpaceDenoise/SSDDefinitions.ush" 


//------------------------------------------------------- CONFIGS

#define DIM_APPLY_DIFFUSE_INDIRECT_SSGI					1
#define DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER	2

#ifndef SSR_TILED_COMPOSITION
#define SSR_TILED_COMPOSITION	0
#endif

#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SSGI
	#define CONFIG_SIGNAL_PROCESSING SIGNAL_PROCESSING_SSGI
	#define COMPILE_SIGNAL_COLOR 1
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10
	#define CONFIG_INPUT_TEXTURE_COUNT 2

#elif DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
	#define CONFIG_SIGNAL_PROCESSING SIGNAL_PROCESSING_SSGI
	#define COMPILE_SIGNAL_COLOR_ARRAY 3
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10
	#define CONFIG_INPUT_TEXTURE_COUNT 4

#else // !DIM_APPLY_DIFFUSE_INDIRECT
	// NOP
#endif // !DIM_APPLY_DIFFUSE_INDIRECT

#define COMPILE_MOMENT1_ACCUMULATOR 1
#define COMPILE_BOX_KERNEL 1

#if SUBSTRATE_ENABLED && (DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SSGI || DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER)
  #if SUBSTRATE_TILETYPE == 0
	#define SUBSTRATE_FASTPATH 1
  #elif SUBSTRATE_TILETYPE == 1
	#define SUBSTRATE_SINGLEPATH 1
  #elif SUBSTRATE_TILETYPE == 2
	// COMPLEX PATH
  #elif SUBSTRATE_TILETYPE == 3
	// COMPLEX PATH
	#define SUBSTRATE_COMPLEXSPECIALPATH 1
  #else
	#error Substrate tile type non-implemented
  #endif
#endif

#if SUBSTRATE_GBUFFER_FORMAT==1 && SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
RWTexture2D<float3> OutOpaqueRoughRefractionSceneColor;
RWTexture2D<float3> OutSubSurfaceSceneColor;
#endif

// Remove a branch of code that is unused where TransformSignal() should always early exit.
// SPIRV does not recognize that the code should be remove which adds an unnecessary parameter requirement.
#define FORCE_IDENTICAL_COLOR_SPACE 1

//------------------------------------------------------- INCLUDES

#include "Common.ush"
#include "SceneTextureParameters.ush"
#include "BRDF.ush"
#include "ShadingModels.ush"
#include "ClearCoatCommon.ush"
#include "FastMath.ush"
#include "Lumen/LumenMaterial.ush"
#include "Lumen/LumenReflectionsCombine.ush"
#include "Substrate/Substrate.ush"
#include "Substrate/SubstrateEvaluation.ush"
#include "HairStrands/HairStrandsEnvironmentLightingCommon.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

#if DIM_APPLY_DIFFUSE_INDIRECT

#include "ScreenSpaceDenoise/SSDSignalFramework.ush"
#include "ScreenSpaceDenoise/SSDSignalArray.ush"
#include "ScreenSpaceDenoise/SSDSpatialKernel.ush"
#include "Lumen/LumenScreenSpaceBentNormal.ush"
#include "Lumen/LumenUpsample.ush"
#include "BlueNoise.ush"

#endif

#if SSR_TILED_COMPOSITION
#include "ScreenSpaceReflectionTileCommons.ush"
#endif
//------------------------------------------------------- PARAMETERS

float AmbientOcclusionStaticFraction;
uint bVisualizeDiffuseIndirect;

Texture2D AmbientOcclusionTexture;
SamplerState AmbientOcclusionSampler;


// For Lumen we use explicit binding with texture arrays
#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
Texture2DArray DiffuseIndirect_Lumen_0;
Texture2DArray DiffuseIndirect_Lumen_1;
Texture2DArray DiffuseIndirect_Lumen_2;
Texture2DArray DiffuseIndirect_Lumen_3;
#endif

Texture2D DiffuseIndirect_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
Texture2D DiffuseIndirect_Textures_1;
#else
#define DiffuseIndirect_Textures_1 DiffuseIndirect_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
Texture2D DiffuseIndirect_Textures_2;
#else
#define DiffuseIndirect_Textures_2 DiffuseIndirect_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
Texture2D DiffuseIndirect_Textures_3;
#else
#define DiffuseIndirect_Textures_3 DiffuseIndirect_Textures_0
#endif

#if !ENABLE_DUAL_SRC_BLENDING
Texture2D SceneColorTexture;
SamplerState SceneColorSampler;
#endif

RWTexture2D<float4> PassDebugOutput;

uint bLumenSupportBackfaceDiffuse;
uint bLumenReflectionInputIsSSR;
uint LumenGIDownsampleFactor;
uint ShortRangeAODownsampleFactor;
float LumenFoliageOcclusionStrength;
float LumenMaxAOMultibounceAlbedo;
float LumenReflectionContrast;
float LumenReflectionSpecularScale;
float LumenAmbientOcclusionIntensity;

#if SSR_TILED_COMPOSITION
StructuredBuffer<uint> SSRTileMaskBuffer;
uint2 SSRTiledViewRes;
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions

struct FShadingOcclusion
{
	float SpecularOcclusion;
	float3 DiffuseOcclusion;
};

#if DIM_SCREEN_BENT_NORMAL_MODE == 2
	Texture2DArray<uint> ShortRangeAOTexture;
#else
	Texture2DArray<UNORM float> ShortRangeAOTexture;
#endif

Texture2DArray<float3> ShortRangeGITexture;

float3 GetShortRangeGI(uint2 PixelPos)
{
	float3 Lighting = 0;

#if DIM_SHORT_RANGE_GI
	Lighting = ShortRangeGITexture[uint3(PixelPos, 0)];
#endif

	return Lighting;
}

FShadingOcclusion GetShadingOcclusion(uint2 PixelPos, uint2 LumenBufferCoord, uint InBSDFIndex, float3 V, float3 N, float Roughness, float3 DiffuseColor, float InAO, bool bIsFoliage)
{
	FShadingOcclusion Out = (FShadingOcclusion)0;
	Out.SpecularOcclusion = 1;
	Out.DiffuseOcclusion = 1;
	
	uint2 ShortRangeAOCoord = ShortRangeAODownsampleFactor == LumenGIDownsampleFactor ? LumenBufferCoord : PixelPos;

#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
  #if DIM_SCREEN_BENT_NORMAL_MODE == 2
	const float3 BentNormal = UnpackScreenBentNormal(ShortRangeAOTexture[uint3(ShortRangeAOCoord, InBSDFIndex)]);
	const float AO = saturate(length(BentNormal));
	Out.SpecularOcclusion = CalculateSpecularOcclusion(N, Roughness, AO, V, BentNormal);
	Out.DiffuseOcclusion = AOMultiBounce(min(DiffuseColor, LumenMaxAOMultibounceAlbedo), AO);
  #elif DIM_SCREEN_BENT_NORMAL_MODE == 1
	const float3 BentNormal = N;
	const float AO = ShortRangeAOTexture[uint3(ShortRangeAOCoord, InBSDFIndex)];
	Out.SpecularOcclusion = CalculateSpecularOcclusion(N, Roughness, AO, V, BentNormal);
	Out.DiffuseOcclusion = AOMultiBounce(min(DiffuseColor, LumenMaxAOMultibounceAlbedo), AO);
  #else
	Out.SpecularOcclusion = InAO;
	Out.DiffuseOcclusion = InAO;
  #endif

	Out.DiffuseOcclusion = lerp(1, Out.DiffuseOcclusion, LumenAmbientOcclusionIntensity);

	if (bIsFoliage)
	{
		Out.DiffuseOcclusion = lerp(1, Out.DiffuseOcclusion, LumenFoliageOcclusionStrength);
	}

#else
	Out.DiffuseOcclusion = 1;
#endif

	return Out;
}

float3 AddContrastAndSpecularScale(float3 In)
{
	// Constrast adjustment with mid-grey as 'anchor' point so that overall exposure don't change
	In = pow(In * (1.f / 0.18f), LumenReflectionContrast) * 0.18f;
	return In * LumenReflectionSpecularScale;
}

float3 MixSpecularAndRoughReflections(float InRoughness, bool bHasBackfaceDiffuse, float4 SpecularReflections, float3 RoughReflections)
{
	const float FadeAlpha = LumenCombineReflectionsAlpha(InRoughness, bHasBackfaceDiffuse);
	float3 Lighting = bLumenReflectionInputIsSSR ? RoughReflections : RoughReflections * (1 - FadeAlpha);

	if (bLumenReflectionInputIsSSR > 0)
	{
		// #lumen_todo: this incorrectly blends over ScreenProbeGather direct specular
		Lighting = lerp(Lighting, SpecularReflections.rgb, SpecularReflections.a);
	}
	// Must branch as SpecularReflections can be uninitialized where not needed and contain NaN
	else if (FadeAlpha > 0.0f)
	{
		Lighting += SpecularReflections.rgb * FadeAlpha;
	}
	
	return AddContrastAndSpecularScale(Lighting);
}

float3 CombineRoughSpecular(FGBufferData GBuffer, bool bHasBackfaceDiffuse, float NoV, float4 RayTracedReflections, float3 RoughReflections, float3 SpecularColor, float3 EnvBRDF)
{
	float3 Lighting;
	
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		Lighting = ClearCoatLayerCombine(GBuffer, NoV, AddContrastAndSpecularScale(RayTracedReflections.xyz), AddContrastAndSpecularScale(RoughReflections), SpecularColor);
	}
	else
	{
		Lighting = MixSpecularAndRoughReflections(GBuffer.Roughness, bHasBackfaceDiffuse, RayTracedReflections, RoughReflections) * EnvBRDF;
	}
	return Lighting;
}

float GetSSSCheckerboadSpecularScale(uint2 PixelPos, bool bNeedsSeparateLightAccumulation)
{
	float SpecularScale = 1.0f;
	if (bNeedsSeparateLightAccumulation &&
		View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0)
	{
		bool bChecker = CheckerFromPixelPos(PixelPos);
		// Adjust for checkerboard. only apply non-diffuse lighting (including emissive) 
		// to the specular component, otherwise lighting is applied twice
		SpecularScale = !bChecker;
	}
	return SpecularScale;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Lumen diffuse evaluation for Substrate material.

#if SUBSTRATE_GBUFFER_FORMAT==1
FSubstrateDeferredLighting SubstrateIndirectLighting(
	uint2 PixelPos,
	uint2 LumenBufferCoord,
	FSubstrateMaterialContainer MaterialBuffer,
	inout FSubstrateAddressing SubstrateAddressing,
	FSubstratePixelHeader SubstratePixelHeader,
	in float3 V,
	in float  InAO,
	in float3 InputDiffuseLighting,
	in float4 InputSpecularLighting)
{
	FSubstrateDeferredLighting Out = GetInitialisedSubstrateDeferredLighting();
	const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);

	Substrate_for(uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
	{
		// * Lumen integration output indirect lighting per BSDF
		// * SSGI  integration output a single indirect lighting value for all BSDF
	#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
		const uint4 LumenFetchCoord				= uint4(LumenBufferCoord, ClosureIndex, 0);
		const float3 InDiffuseLighting			= DiffuseIndirect_Lumen_0.Load(LumenFetchCoord).rgb;
		const float3 InRoughSpecularLighting	= DiffuseIndirect_Lumen_2.Load(LumenFetchCoord).rgb;
		const uint4 LumenSpecularFetchCoord		= uint4(PixelPos, ClosureIndex, 0);
		const float4 InSpecularLighting			= bLumenReflectionInputIsSSR ? InputSpecularLighting : DiffuseIndirect_Lumen_3.Load(LumenSpecularFetchCoord).rgba;
	#else
		const float3 InDiffuseLighting			= InputDiffuseLighting;
		const float3 InRoughSpecularLighting	= InputSpecularLighting.xyz;
		const float4 InSpecularLighting			= InputSpecularLighting;
	#endif

		FSubstrateBSDF BSDF = UnpackSubstrateBSDF(MaterialBuffer, SubstrateAddressing, SubstratePixelHeader);
		if (bVisualizeDiffuseIndirect)
		{
			SubstrateSetBSDFDiffuseColor(BSDF, float3(0.18f, 0.18f, 0.18f));
		}

		// Create the BSDF context
		FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V);
		const float3 BSDFThroughput = LuminanceWeight(SubstrateBSDFContext, BSDF);

		// Evaluate environment lighting
		FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);
		
		const float3 DiffuseColor = SubstrateGetBSDFDiffuseColor(BSDF);
		const float  Roughness = SubstrateGetBSDFRoughness(BSDF);
		const bool bHasBackfaceDiffuse = SubstrateGetBSDFType(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && BSDF.HasBackScattering();
		const FShadingOcclusion Occlusion = GetShadingOcclusion(PixelPos, LumenBufferCoord, ClosureIndex, V, SubstrateBSDFContext.N, Roughness, DiffuseColor, InAO, bHasBackfaceDiffuse);

		const float3 InGlossyLighting = MixSpecularAndRoughReflections(Roughness, bHasBackfaceDiffuse, InSpecularLighting, InRoughSpecularLighting * Occlusion.SpecularOcclusion);

		float3 ShortRangeGI = GetShortRangeGI(PixelPos);

		FDirectLighting OutBSDF = (FDirectLighting)0;
		OutBSDF.Diffuse  += BSDFThroughput * SubstrateEnvLight.DiffuseWeight * (InDiffuseLighting * Occlusion.DiffuseOcclusion + ShortRangeGI);
		OutBSDF.Specular += BSDFThroughput * InGlossyLighting  * SubstrateEnvLight.SpecularWeight; // Note: Occlusion.SpecularOcclusion is only applied on the rough specular part, as in legacy path.

		#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
		if (bHasBackfaceDiffuse && bLumenSupportBackfaceDiffuse > 0)
		{
			const float3 InBackfaceDiffuseLighting = DiffuseIndirect_Lumen_1.Load(LumenFetchCoord).rgb;
			OutBSDF.Diffuse  += BSDFThroughput * InBackfaceDiffuseLighting * SubstrateEnvLight.DiffuseBackFaceWeight * PI * Occlusion.DiffuseOcclusion; // Multiplied by PI as is SubstrateEnvLight.DiffuseBackFaceWeight is divided PI
		}
		#endif 

		#if SUBSTRATE_FASTPATH==0
		if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
		{
			const float3 InHazeGlossyLighting = MixSpecularAndRoughReflections(SubstrateEnvLight.SpecularHazeSafeRoughness, bHasBackfaceDiffuse, InSpecularLighting, InRoughSpecularLighting * Occlusion.SpecularOcclusion);
			OutBSDF.Specular += BSDFThroughput * SubstrateEnvLight.SpecularHazeWeight * InHazeGlossyLighting; // Note: Occlusion.SpecularOcclusion is only applied on the rough specular part, as in legacy path.
		}
		#endif 

		// SSS Checkerboard
		OutBSDF.Specular *= GetSSSCheckerboadSpecularScale(PixelPos, SubstrateEnvLight.bPostProcessSubsurface);

		FLightAccumulator Accumulator = (FLightAccumulator)0;
		LightAccumulator_AddSplit(Accumulator, OutBSDF.Diffuse, OutBSDF.Specular, OutBSDF.Diffuse, 1.f /*CommonMultiplier*/, SubstrateEnvLight.bPostProcessSubsurface);
		AccumulateSubstrateDeferredLighting(Out, Accumulator, SubstrateEnvLight.bPostProcessSubsurface, BSDF_GETISTOPLAYER(BSDF));
	}

	return Out;
}
#endif // SUBSTRATE_GBUFFER_FORMAT==1

#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SSGI

float3 UpsampleLegacySSGI(uint2 PixelPos, float2 SceneBufferUV, FLumenMaterialData Material)
{
	// Sample the output of the denoiser.
	FSSDKernelConfig KernelConfig = CreateKernelConfig();
				
	#if DEBUG_OUTPUT
	{
		KernelConfig.DebugPixelPosition = PixelPos;
		KernelConfig.DebugEventCounter = 0;
	}
	#endif

	// Compile time.
	KernelConfig.bSampleKernelCenter = true;
	KernelConfig.BufferLayout = CONFIG_SIGNAL_INPUT_LAYOUT;
	KernelConfig.bUnroll = true;

	#if DIM_UPSCALE_SSGI
	{
		KernelConfig.SampleSet = SAMPLE_SET_2X2_BILINEAR;
		KernelConfig.BilateralDistanceComputation = SIGNAL_WORLD_FREQUENCY_REF_METADATA_ONLY;
		KernelConfig.WorldBluringDistanceMultiplier = 16.0;
				
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(3);
				
		// SGPRs
		KernelConfig.BufferSizeAndInvSize = View.BufferSizeAndInvSize * float4(0.5, 0.5, 2.0, 2.0);
		KernelConfig.BufferBilinearUVMinMax = View.BufferBilinearUVMinMax;
	}
	#else
	{
		KernelConfig.SampleSet = SAMPLE_SET_1X1;
		KernelConfig.bNormalizeSample = true;
				
		// SGPRs
		KernelConfig.BufferSizeAndInvSize = View.BufferSizeAndInvSize;
		KernelConfig.BufferBilinearUVMinMax = View.BufferBilinearUVMinMax;
	}
	#endif

	// VGPRs
	KernelConfig.BufferUV = SceneBufferUV; 
	{
		// SUBSTRATE_TODO: We use the top layer data, but we should resolve lighting for each BSDFs.
		KernelConfig.CompressedRefSceneMetadata = MaterialToCompressedSceneMetadata(Material.SceneDepth, Material.WorldNormal, Material.Roughness, Material.ShadingID);
		KernelConfig.RefBufferUV = SceneBufferUV;
		KernelConfig.RefSceneMetadataLayout = METADATA_BUFFER_LAYOUT_DISABLED;
	}
	KernelConfig.HammersleySeed = Rand3DPCG16(int3(PixelPos, View.StateFrameIndexMod8)).xy;
				
	FSSDSignalAccumulatorArray UncompressedAccumulators = CreateSignalAccumulatorArray();
	FSSDCompressedSignalAccumulatorArray CompressedAccumulators = CompressAccumulatorArray(
		UncompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);

	AccumulateKernel(
		KernelConfig,
		DiffuseIndirect_Textures_0,
		DiffuseIndirect_Textures_1,
		DiffuseIndirect_Textures_2,
		DiffuseIndirect_Textures_3,
		/* inout */ UncompressedAccumulators,
		/* inout */ CompressedAccumulators);

	//PassDebugOutput[PixelPos] = float4(UncompressedAccumulators.Array[0].Moment1.SampleCount, 0, 0, 0);

	FSSDSignalSample Sample;
	#if DIM_UPSCALE_SSGI
		Sample = NormalizeToOneSample(UncompressedAccumulators.Array[0].Moment1);
	#else
		Sample = UncompressedAccumulators.Array[0].Moment1;
	#endif
				
	return Sample.SceneColor.rgb;
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void MainPS(
	float4 SvPosition : SV_POSITION
#if DIM_APPLY_DIFFUSE_INDIRECT
#if ENABLE_DUAL_SRC_BLENDING
	, out float4 OutAddColor      DUAL_SOURCE_BLENDING_SLOT(0) : SV_Target0
	, out float4 OutMultiplyColor DUAL_SOURCE_BLENDING_SLOT(1) : SV_Target1
#else
	, out float4 OutColor 		  : SV_Target0
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	, out float OutSeparatedSubsurfaceDiffuse 		  : SV_Target1
#endif
#endif
#else
	, out float4 OutMultiplyColor : SV_Target0
#endif
)
{
	const uint2 PixelPos = uint2(SvPosition.xy);
	const float2 SceneBufferUV = SvPositionToBufferUV(SvPosition);
	const float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;

	#if !ENABLE_DUAL_SRC_BLENDING && DIM_APPLY_DIFFUSE_INDIRECT
	float4 OutAddColor = float4(0.0, 0.0, 0.0, 0.0);
	float4 OutMultiplyColor;

	const float4 SceneColor = SceneColorTexture.SampleLevel(SceneColorSampler, SceneBufferUV, 0);
	#endif

	// Sample scene textures.
	const FLumenMaterialData Material = ReadMaterialData(PixelPos, SceneBufferUV);

	// Sample the ambient occlusion that is dynamically generated every frame.
	const float DynamicAmbientOcclusion = AmbientOcclusionTexture.SampleLevel(AmbientOcclusionSampler, SceneBufferUV, 0).r;

	// Compute the final ambient occlusion to be applied. Lumen handles material AO internally.
	float FinalAmbientOcclusion = 1.0f;
	#if DIM_APPLY_DIFFUSE_INDIRECT != DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
	{
		float AOMask = IsValid(Material);
		FinalAmbientOcclusion = lerp(1.0f, Material.MaterialAO * DynamicAmbientOcclusion, AOMask * AmbientOcclusionStaticFraction);
	}
	#endif

	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, Material.SceneDepth), Material.SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;

	const float3 N = Material.WorldNormal;
	const float3 V = -GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	const float NoV = saturate(dot(N, V));

	// Apply diffuse indirect.
	#if DIM_APPLY_DIFFUSE_INDIRECT
	OutAddColor = 0;
	{
		FDirectLighting IndirectLighting = (FDirectLighting)0;

		if (IsValid(Material))
		{
			float3 DiffuseIndirectLighting = 0;
			float3 RoughSpecularIndirectLighting = 0;
			float4 SpecularIndirectLighting = 0;
			uint2 LumenBufferCoord = PixelPos;

			#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
				#if DIM_UPSCALE_LUMEN_GI
				{
					const float RandomScalar = BlueNoiseScalar(PixelPos, View.StateFrameIndex);
					uint2 StochasticBilinearOffset = GetStochasticBilinearOffsetForUpsample(PixelPos, SceneBufferUV, TranslatedWorldPosition, Material.WorldNormal, Material.SceneDepth, RandomScalar, LumenGIDownsampleFactor);
					int2 UnclampedDownsampledCoord = (int2(PixelPos) - 1) / LumenGIDownsampleFactor + StochasticBilinearOffset;
					LumenBufferCoord = clamp(UnclampedDownsampledCoord, (int2)IntegrateViewMin, (int2)(IntegrateViewMin + IntegrateViewSize - 1));
				}
				#endif

				DiffuseIndirectLighting = DiffuseIndirect_Lumen_0.Load(uint4(LumenBufferCoord, 0, 0)).rgb;
				RoughSpecularIndirectLighting = DiffuseIndirect_Lumen_2.Load(uint4(LumenBufferCoord, 0, 0)).rgb;

				if (bLumenReflectionInputIsSSR)
				{
					#if SSR_TILED_COMPOSITION
					if (IsSSRTileUsed(PixelPos, SSRTiledViewRes, SSRTileMaskBuffer))
					#endif
					{
						SpecularIndirectLighting = DiffuseIndirect_Textures_3.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0).rgba;
					}
				}
				else
				{
					SpecularIndirectLighting = DiffuseIndirect_Lumen_3.SampleLevel(GlobalPointClampedSampler, float3(SceneBufferUV, 0), 0).rgba;
				}
			#elif DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SSGI
				DiffuseIndirectLighting = UpsampleLegacySSGI(PixelPos, SceneBufferUV, Material);
			#endif

			#if SUBSTRATE_GBUFFER_FORMAT==1
			{
				FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
				FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
				if (SubstratePixelHeader.GetMaterialMode() > HEADER_MATERIALMODE_NONE)
				{
					const FSubstrateDeferredLighting IndirectLighting_Substrate = SubstrateIndirectLighting(
						PixelPos,
						LumenBufferCoord,
						Substrate.MaterialTextureArray,
						SubstrateAddressing,
						SubstratePixelHeader,
						V,
						DynamicAmbientOcclusion,
						DiffuseIndirectLighting,
						SpecularIndirectLighting);

					OutAddColor = IndirectLighting_Substrate.SceneColor;
				#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
					const uint2 OutCoord = PixelPos;
					OutOpaqueRoughRefractionSceneColor[OutCoord] += IndirectLighting_Substrate.OpaqueRoughRefractionSceneColor;
					OutSubSurfaceSceneColor[OutCoord] += IndirectLighting_Substrate.SubSurfaceSceneColor;
				#endif
				}
			}
			#else // SUBSTRATE_GBUFFER_FORMAT==1
			{
			FGBufferData GBuffer = Material.GBufferData;

			float3 DiffuseColor = bVisualizeDiffuseIndirect ? float3(.18f, .18f, .18f) : GBuffer.DiffuseColor;
			float3 SpecularColor = GBuffer.SpecularColor;

			#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
				RemapClearCoatDiffuseAndSpecularColor(GBuffer, NoV, DiffuseColor, SpecularColor);
			#endif

			bool bIsFoliage = GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE || GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE;
			FShadingOcclusion Occlusion = GetShadingOcclusion(PixelPos, LumenBufferCoord, 0 /*BSDFIndex*/, V, N, GBuffer.Roughness, GBuffer.BaseColor, DynamicAmbientOcclusion, bIsFoliage);

			if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
			{
				float3 L = 0;
				const float3 IndirectDiffuseColor = EvaluateEnvHair(GBuffer, V, N, L /*out*/);
				IndirectLighting.Diffuse = DiffuseIndirectLighting * Occlusion.DiffuseOcclusion * IndirectDiffuseColor;
				IndirectLighting.Specular = 0;
			}
			else
			{
				float3 BackfaceDiffuseIndirectLighting = 0;

				if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
				{
					float3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);

					if (bLumenSupportBackfaceDiffuse > 0)
					{
						#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
						BackfaceDiffuseIndirectLighting += SubsurfaceColor * DiffuseIndirect_Lumen_1.SampleLevel(GlobalPointClampedSampler, float3(SceneBufferUV, 0), 0).rgb;
						#else
						BackfaceDiffuseIndirectLighting += SubsurfaceColor * DiffuseIndirect_Textures_1.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0).rgb;
						#endif
					}
					else
					{
						// Adding Subsurface energy to the diffuse lobe is a poor approximation when DiffuseColor is small and SubsurfaceColor is large
						// Reduce the error by attenuating SubsurfaceColor, even though diffuse already has the 1/PI for Lambert.
						const float PreintegratedTwoSidedBxDF = 1.0f / PI;
						DiffuseColor += SubsurfaceColor * PreintegratedTwoSidedBxDF;
					}
				}

				if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
				{
					float3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
					// Add subsurface energy to diffuse
					DiffuseColor += SubsurfaceColor;
				}

				if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
				{
					float3 ClothFuzz = ExtractSubsurfaceColor(GBuffer);
					DiffuseColor += ClothFuzz * GBuffer.CustomData.a;
				}

			#if SUBSTRATE_ENABLED
				float ToonProfileId = 0.0f;
				// This path is not calling SubstrateEvaluateForEnvLight so we need to add that specialised code here.
				BRANCH
				if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
				{
					float2 PatternUVs = 0.0;
					UnpackToonCustomData(PackRGBA8(GBuffer.CustomData), PatternUVs, ToonProfileId);

					FToonProfileGISettings ToonProfileGISettings = GetToonProfileGISettings(ToonProfileId);
					DiffuseColor *= ToonProfileGISettings.DiffuseIndirectScale;
					SpecularColor *= ToonProfileGISettings.SpecularIndirectScale;
				}
			#endif

			#if USE_ENERGY_CONSERVATION
				FBxDFEnergyTerms SpecularEnergyTerms = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, NoV, SpecularColor);
				float3 EnvBRDFValue = SpecularEnergyTerms.E; // EnvBRDF accounting for multiple scattering when enabled
				float EnergyPreservationFactor = ComputeEnergyPreservation(SpecularEnergyTerms);
			#else
				float3 EnvBRDFValue = EnvBRDF(SpecularColor, GBuffer.Roughness, NoV);
				float EnergyPreservationFactor = 1.0;
			#endif

				float3 ShortRangeGI = GetShortRangeGI(PixelPos);

				IndirectLighting.Diffuse = (DiffuseIndirectLighting * DiffuseColor + BackfaceDiffuseIndirectLighting) * Occlusion.DiffuseOcclusion * EnergyPreservationFactor + ShortRangeGI * DiffuseColor;
				IndirectLighting.Transmission = 0;

				#if DIM_APPLY_DIFFUSE_INDIRECT == DIM_APPLY_DIFFUSE_INDIRECT_SCREEN_PROBE_GATHER
					RoughSpecularIndirectLighting *= Occlusion.SpecularOcclusion;
					IndirectLighting.Specular = CombineRoughSpecular(GBuffer, HasBackfaceDiffuse(Material), NoV, SpecularIndirectLighting, RoughSpecularIndirectLighting, SpecularColor, EnvBRDFValue);
				#else
					IndirectLighting.Specular = AddContrastAndSpecularScale(SpecularIndirectLighting.xyz) * EnvBRDFValue;
				#endif

			#if SUBSTRATE_ENABLED && SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION
				IndirectLighting.Specular = SubstrateToonQuantize(ToonProfileId, IndirectLighting.Specular * View.OneOverPreExposure) * View.PreExposure;
			#endif
			}
			}
			#endif // SUBSTRATE_GBUFFER_FORMAT==1
		}

		// Accumulate lighting into the final buffers
	#if SUBSTRATE_GBUFFER_FORMAT==0
		IndirectLighting.Specular *= GetSSSCheckerboadSpecularScale(PixelPos, Material.bNeedsSeparateLightAccumulation);
		FLightAccumulator LightAccumulator = (FLightAccumulator)0;
		LightAccumulator_Add(
			LightAccumulator,
			IndirectLighting.Diffuse + IndirectLighting.Specular,
			IndirectLighting.Diffuse,
			1.0f,
			Material.bNeedsSeparateLightAccumulation);
		OutAddColor = LightAccumulator_GetResult(LightAccumulator);
	#endif // SUBSTRATE_GBUFFER_FORMAT==0

	}
	#endif

	OutMultiplyColor = FinalAmbientOcclusion;

	#if !ENABLE_DUAL_SRC_BLENDING && DIM_APPLY_DIFFUSE_INDIRECT
	OutColor = SceneColor * OutMultiplyColor + OutAddColor;
	#endif

	#if DIM_APPLY_DIFFUSE_INDIRECT && SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	    #if !ENABLE_DUAL_SRC_BLENDING
		   // TODO: When dual src blending is not enabled, multiply OutMultiplyColor to the old Subsurface diffuse accumulation.
	       OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
        #else 
	       // Dual source does not support MRT. Use UAV.
		   AccumulateSeparatedSubsurfaceDiffuse(uint2(SvPosition.xy), OutAddColor, FinalAmbientOcclusion);
		#endif
	#endif
}
