// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DisplayMapping.usf: map input texture to output texture color space.
=============================================================================*/

#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "TonemapCommon.ush"

#ifndef DISPLAY_MAPPING_INPUT_IS_LINEAR
	#define DISPLAY_MAPPING_INPUT_IS_LINEAR 0
#endif // DISPLAY_MAPPING_INPUT_IS_LINEAR

#ifndef DISPLAY_MAPPING_PS_FROM_ARRAY
	#define DISPLAY_MAPPING_PS_FROM_ARRAY 0
#endif // DISPLAY_MAPPING_PS_FROM_ARRAY

#if DISPLAY_MAPPING_PS_FROM_ARRAY
	uint ArraySlice;
	#define TextureOrArray2D						Texture2DArray
	#define TexOrArr2DSample(Tex, S, UV)			Tex.Sample(S, float3(UV.x, UV.y, ArraySlice))
	#define TexOrArr2DSampleLevel(Tex, S, UV, L)	Tex.SampleLevel(S, float3(UV.x, UV.y, ArraySlice), L)
#else
	#define TextureOrArray2D						Texture2D
	#define TexOrArr2DSample(Tex, S, UV)			Texture2DSample(Tex, S, UV)
	#define TexOrArr2DSampleLevel(Tex, S, UV, L)	Texture2DSampleLevel(Tex, S, UV, L)
#endif

uint OutputDevice;
uint OutputGamut;
float OutputMaxLuminance;

TextureOrArray2D SceneTexture;
SamplerState SceneSampler;
float4x4 TextureToOutputGamutMatrix;

float3 GetLinearColor(float3 InColor)
{
#if DISPLAY_MAPPING_INPUT_IS_LINEAR
    return InColor;
#else
    return sRGBToLinear(InColor);
#endif
}

void DisplayMappingPS(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	const float4 SceneColor = TexOrArr2DSampleLevel(SceneTexture, SceneSampler, Input.UV, 0);
	OutColor = SceneColor;

	float3 LinearSceneColor = GetLinearColor(SceneColor.rgb);
	LinearSceneColor = mul((float3x3)TextureToOutputGamutMatrix, LinearSceneColor);
	
	if(OutputDevice == TONEMAPPER_OUTPUT_ACES1000nitST2084 || OutputDevice == TONEMAPPER_OUTPUT_ACES2000nitST2084)
	{
		OutColor.xyz = LinearToST2084(LinearSceneColor*300.f);
	}
	else
	{
		OutColor.xyz = LinearToSrgbBranchless(saturate(LinearSceneColor));
	}
}