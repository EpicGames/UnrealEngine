// Copyright Epic Games, Inc. All Rights Reserved.

#define USE_DISTANCE_FIELD_SAMPLER 1

#include "../Common.ush"
#include "../ComputeShaderUtils.ush"
#include "../DeferredShadingCommon.ush"
#include "../DistanceFieldLightingShared.ush"
#include "../DistanceFieldAOShared.ush"
#include "../MeshDistanceFieldCommon.ush"
#include "GlobalDistanceFieldUpdate.ush"
#include "GlobalDistanceFieldShared.ush"
#include "GlobalDistanceFieldUtils.ush"

#ifndef CULLOBJECTS_THREADGROUP_SIZE
	#define CULLOBJECTS_THREADGROUP_SIZE 1
#endif

#ifndef THREADGROUP_SIZE
	#define THREADGROUP_SIZE 1
#endif

#ifndef THREADGROUP_SIZE_X
	#define THREADGROUP_SIZE_X 1
	#define THREADGROUP_SIZE_Y 1
	#define THREADGROUP_SIZE_Z 1
#endif

// Must match FPackedClipmapData in GlobalDistanceField.cpp
struct FPackedClipmapData
{
	float3 TranslatedWorldCenter; // Camera-centered world space
	float MeshSDFRadiusThreshold;	

	float3 WorldExtent;
	float InfluenceRadiusSq;

	uint ClipmapIndex; 
	uint Flags;
	uint Padding0;
	uint Padding1;	
};

struct FPackedClipmap
{
	float3 TranslatedWorldCenter; // Camera-centered world space
	float3 WorldExtent;

	float MeshSDFRadiusThreshold;
	float InfluenceRadiusSq;

	uint ClipmapIndex; // Global index
	bool bReadbackHasPendingStreaming;
};

StructuredBuffer<FPackedClipmapData> PackedClipmapBuffer;

FPackedClipmap GetPackedClipmap(uint PackedClipmapIndex)
{
	FPackedClipmapData ClipmapData = PackedClipmapBuffer[PackedClipmapIndex];

	FPackedClipmap PackedClipmap;

	PackedClipmap.TranslatedWorldCenter = ClipmapData.TranslatedWorldCenter;
	PackedClipmap.WorldExtent = ClipmapData.WorldExtent;

	PackedClipmap.MeshSDFRadiusThreshold = ClipmapData.MeshSDFRadiusThreshold;
	PackedClipmap.InfluenceRadiusSq = ClipmapData.InfluenceRadiusSq;

	PackedClipmap.ClipmapIndex = ClipmapData.ClipmapIndex;
	PackedClipmap.bReadbackHasPendingStreaming = ClipmapData.Flags != 0;

	return PackedClipmap;
}

RWStructuredBuffer<uint> RWCullGridAllocator;
RWStructuredBuffer<uint> RWCullGridObjectHeader;
RWStructuredBuffer<uint> RWCullGridObjectArray;

#define CULL_GRID_FACTOR 4
#define GRID_CULL_THREADGROUP_TOTALSIZE 256

uint3 UpdateRegionSize;
StructuredBuffer<uint> CullGridObjectHeader;
StructuredBuffer<uint> CullGridObjectArray;

RWStructuredBuffer<uint> RWObjectIndexBuffer;
RWStructuredBuffer<uint> RWObjectIndexNumBuffer;

RWStructuredBuffer<uint> RWHasPendingStreaming;


float3 PreViewTranslationHigh;
float3 PreViewTranslationLow;

// 0: accept only mostly static objects, 1: accept only often moving objects, 2: accept all objects
uint AcceptOftenMovingObjectsOnly;
uint NumPackedClipmaps;
uint ObjectIndexBufferStride;

#ifdef CullObjectsToClipmapCS

[numthreads(CULLOBJECTS_THREADGROUP_SIZE, 1, 1)]
void CullObjectsToClipmapCS(
	uint GroupIndex : SV_GroupIndex,
	uint3 GroupId : SV_GroupID)
{
	const uint ObjectIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, CULLOBJECTS_THREADGROUP_SIZE);
	if (ObjectIndex < NumSceneObjects)
	{
		const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);

		const bool bMatchingMobility = AcceptOftenMovingObjectsOnly == 2 || DFObjectBounds.OftenMoving == AcceptOftenMovingObjectsOnly;
		const bool bVisible = DFObjectBounds.bVisible || DFObjectBounds.bAffectIndirectLightingWhileHidden;

		if (bMatchingMobility && bVisible)
		{
			const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);
			const float3 TranslatedWorldObjectBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);

			for (uint PackedClipmapIndex = 0; PackedClipmapIndex < NumPackedClipmaps; ++PackedClipmapIndex)
			{
				FPackedClipmap Clipmap = GetPackedClipmap(PackedClipmapIndex);

				if (DFObjectBounds.SphereRadius > Clipmap.MeshSDFRadiusThreshold || DFObjectBounds.bEmissiveLightSource)
				{
					float DistanceSq = ComputeSquaredDistanceBetweenAABBs(Clipmap.TranslatedWorldCenter, Clipmap.WorldExtent, TranslatedWorldObjectBoundsCenter, DFObjectBounds.BoxExtent);

					if (DistanceSq < Clipmap.InfluenceRadiusSq)
					{
						uint DestIndex;
						InterlockedAdd(RWObjectIndexNumBuffer[PackedClipmapIndex], 1U, DestIndex);
						RWObjectIndexBuffer[DestIndex + PackedClipmapIndex * ObjectIndexBufferStride] = ObjectIndex;

						#if READBACK_HAS_PENDING_STREAMING
						if (Clipmap.bReadbackHasPendingStreaming)
						{
							FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
							uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;

							if (NumMips == 1)
							{
								RWHasPendingStreaming[Clipmap.ClipmapIndex] = 1;
							}
						}
						#endif
					}
				}
			}
		}
	}
}

#endif

RWBuffer<uint> RWPageComposeIndirectArgBuffer;
RWBuffer<uint> RWPageUpdateIndirectArgBuffer;
RWBuffer<uint> RWCullGridUpdateIndirectArgBuffer;
RWBuffer<uint> RWBuildObjectGridIndirectArgBuffer;


#ifdef ClearIndirectArgBufferCS

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ClearIndirectArgBufferCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (all(DispatchThreadId) == 0)
	{
		const uint3 ClearValue = uint3(0, 1, 1);
		WriteDispatchIndirectArgs(RWPageComposeIndirectArgBuffer, 0, ClearValue);
		WriteDispatchIndirectArgs(RWPageUpdateIndirectArgBuffer, 0, ClearValue);
		WriteDispatchIndirectArgs(RWCullGridUpdateIndirectArgBuffer, 0, ClearValue);
		WriteDispatchIndirectArgs(RWBuildObjectGridIndirectArgBuffer, 0, ClearValue);
	}
}

#endif

RWStructuredBuffer<uint> RWPageTileBuffer;
RWBuffer<uint> RWPageIndirectArgBuffer;

RWStructuredBuffer<uint> RWCullGridTileBuffer;
RWBuffer<uint> RWCullGridIndirectArgBuffer;

Buffer<float4> UpdateBoundsBuffer;
uint NumUpdateBounds;
float InfluenceRadiusSq;

uint3 CullGridResolution;
float3 CullGridCoordToTranslatedWorldCenterScale;
float3 CullGridCoordToTranslatedWorldCenterBias;
float3 CullGridTileWorldExtent;

float3 PageGridCoordToTranslatedWorldCenterScale;
float3 PageGridCoordToTranslatedWorldCenterBias;
float3 PageGridTileWorldExtent;

#ifdef BuildGridTilesCS

#define MAX_CULLED_TILES 511u
groupshared uint SharedCulledBoundsList[MAX_CULLED_TILES];
groupshared uint SharedNumCulledBounds;

// 4x4x4 bitmask with occupied cells
groupshared uint SharedCullGridMask[2];

void SetSharedCullGridMaskBit(uint3 LocalGridCoord)
{
	uint BitIndex = LocalGridCoord.x + (LocalGridCoord.y + LocalGridCoord.z * CULL_GRID_FACTOR) * CULL_GRID_FACTOR;
	if (BitIndex >= 32)
	{
		BitIndex -= 32;
		InterlockedOr(SharedCullGridMask[1], 1u << BitIndex);
	}
	else
	{
		InterlockedOr(SharedCullGridMask[0], 1u << BitIndex);
	}
}

bool IsSharedCullGridMaskBitSet(uint3 LocalGridCoord)
{
	uint BitIndex = LocalGridCoord.x + (LocalGridCoord.y + LocalGridCoord.z * CULL_GRID_FACTOR) * CULL_GRID_FACTOR;
	if (BitIndex >= 32)
	{
		BitIndex -= 32;
		return (SharedCullGridMask[1] & (1u << BitIndex)) != 0;
	}
	else
	{
		return (SharedCullGridMask[0] & (1u << BitIndex)) != 0;
	}
}

bool IsAnySharedCullGridMaskBitSet()
{
	return SharedCullGridMask[0] != 0 || SharedCullGridMask[1] != 0;
}

/**
 * Run two pass culling to find out which cull and page tiles need to be updated. 
 * First cull into cull tiles, then cull into pages tiles.
 * Every cull tile consists out of 4x4x4 (CULL_GRID_FACTOR) page tiles.
*/
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void BuildGridTilesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint ThreadIndex = GroupThreadId.x + (GroupThreadId.y + GroupThreadId.z * THREADGROUP_SIZE) * THREADGROUP_SIZE;
	if (all(GroupThreadId == 0))
	{
		SharedCullGridMask[0] = 0;
		SharedCullGridMask[1] = 0;
		SharedNumCulledBounds = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	uint3 CullGridCoord = GroupId.xyz;

	// First cull per group
	if (all(CullGridCoord < CullGridResolution))
	{
		float3 CullGridTranslatedWorldCenter = CullGridCoord * CullGridCoordToTranslatedWorldCenterScale + CullGridCoordToTranslatedWorldCenterBias;

		for (uint UpdateBoundsIndex = ThreadIndex; UpdateBoundsIndex < NumUpdateBounds; UpdateBoundsIndex += THREADGROUP_SIZE * THREADGROUP_SIZE * THREADGROUP_SIZE)
		{
			float3 UpdateBoundsCenter = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 0].xyz;
			float3 UpdateBoundsExtent = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 1].xyz;

			bool bExtendByInflueceRadius = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 0].w > 0.0f;
			float UpdateBoundsInfluenceRadiusSq = bExtendByInflueceRadius ? InfluenceRadiusSq : 0.0f;

			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(UpdateBoundsCenter, UpdateBoundsExtent, CullGridTranslatedWorldCenter, CullGridTileWorldExtent);
			if (DistanceSq <= UpdateBoundsInfluenceRadiusSq)
			{
				uint DestIndex;
				InterlockedAdd(SharedNumCulledBounds, 1, DestIndex);
				if (DestIndex < MAX_CULLED_TILES)
				{
					SharedCulledBoundsList[DestIndex] = UpdateBoundsIndex;
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	uint3 PageGridCoord = DispatchThreadId.xyz;
	uint3 LocalPageGridCoord = GroupThreadId.xyz;

	// Then cull per page
	if (all(PageGridCoord < PageGridResolution))
	{
		float3 PageGridTranslatedWorldCenter = PageGridCoord * PageGridCoordToTranslatedWorldCenterScale + PageGridCoordToTranslatedWorldCenterBias;
		const uint NumCulledBounds = min(SharedNumCulledBounds, MAX_CULLED_TILES);

		for (uint IndexInCulledBoundsList = 0; IndexInCulledBoundsList < NumCulledBounds; ++IndexInCulledBoundsList)
		{
			uint UpdateBoundsIndex = SharedCulledBoundsList[IndexInCulledBoundsList];

			float3 UpdateBoundsCenter = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 0].xyz;
			float3 UpdateBoundsExtent = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 1].xyz;

			bool bExtendByInflueceRadius = UpdateBoundsBuffer[UpdateBoundsIndex * 2 + 0].w > 0.0f;
			float UpdateBoundsInfluenceRadiusSq = bExtendByInflueceRadius ? InfluenceRadiusSq : 0.0f;

			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(UpdateBoundsCenter, UpdateBoundsExtent, PageGridTranslatedWorldCenter, PageGridTileWorldExtent);
			if (DistanceSq <= UpdateBoundsInfluenceRadiusSq)
			{
				SetSharedCullGridMaskBit(LocalPageGridCoord);
				break;
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (all(GroupThreadId == 0))
	{
		if (IsAnySharedCullGridMaskBitSet())
		{
			uint DestIndex;
			InterlockedAdd(RWCullGridIndirectArgBuffer[0], 1, DestIndex);
			RWCullGridTileBuffer[DestIndex] = PackPageUpdateTile(CullGridCoord);
		}
	}

	if (IsSharedCullGridMaskBitSet(LocalPageGridCoord))
	{
		uint DestIndex;
		InterlockedAdd(RWPageIndirectArgBuffer[0], 1, DestIndex);
		RWPageTileBuffer[DestIndex] = PackPageUpdateTile(PageGridCoord);
	}
}

#endif

RWStructuredBuffer<uint> RWPageFreeListBuffer;
RWStructuredBuffer<int> RWPageFreeListAllocatorBuffer;
uint GlobalDistanceFieldMaxPageNum;

#ifdef InitPageFreeListCS

[numthreads(THREADGROUP_SIZE, 1, 1)]
void InitPageFreeListCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint FreeListIndex = DispatchThreadId.x;

	if (FreeListIndex < GlobalDistanceFieldMaxPageNum)
	{
		RWPageFreeListBuffer[FreeListIndex] = GlobalDistanceFieldMaxPageNum - 1 - FreeListIndex;
	}

	if (DispatchThreadId.x == 0)
	{
		RWPageFreeListAllocatorBuffer[0] = GlobalDistanceFieldMaxPageNum;
	}
}

#endif

#ifdef AllocatePagesCS

RWTexture3D<uint> RWPageTableCombinedTexture;
RWTexture3D<uint> RWPageTableLayerTexture;
Texture3D<uint> ParentPageTableLayerTexture;

float3 InvPageGridResolution;
float3 PageWorldExtent;
float PageWorldRadius;
float ClipmapInfluenceRadius;

RWStructuredBuffer<uint> RWPageComposeTileBuffer;

RWStructuredBuffer<uint> RWPageFreeListReturnBuffer;
RWStructuredBuffer<uint> RWPageFreeListReturnAllocatorBuffer;

StructuredBuffer<uint> MarkedHeightfieldPageBuffer;
StructuredBuffer<uint> PageUpdateTileBuffer;
StructuredBuffer<uint> PageFreeListBuffer;

groupshared float GroupMinPageDistance[64];

#ifndef PROCESS_DISTANCE_FIELDS
#define PROCESS_DISTANCE_FIELDS 0
#endif // PROCESS_DISTANCE_FIELDS

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void AllocatePagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint ThreadIndex = GroupThreadId.x;
	uint IndexInPageTileBuffer = GroupId.x;
	uint3 PageGridCoord = UnpackPageUpdateTile(PageUpdateTileBuffer[IndexInPageTileBuffer]);

	bool bAnyObjectsInPage = false;
#if MARKED_HEIGHTFIELD_PAGE_BUFFER
	bAnyObjectsInPage = MarkedHeightfieldPageBuffer[IndexInPageTileBuffer] > 0;
#endif

	uint3 PageTableTextureCoord = PageGridCoordToPageTableTextureCoord(PageGridCoord);
	float3 PageTranslatedWorldCenter = PageGridCoord * PageCoordToPageTranslatedWorldCenterScale + PageCoordToPageTranslatedWorldCenterBias;
	FDFVector3 PageWorldCenter = DFFastSubtract(PageTranslatedWorldCenter, PreViewTranslation);

#if PROCESS_DISTANCE_FIELDS
	// Loop through objects to find out if the current page contains any objects and should be allocated
	if (!bAnyObjectsInPage)
	{
		float InfluenceMax = ClipmapInfluenceRadius + PageWorldRadius + GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
		float MinDistanceToPageCenter = InfluenceMax;

		uint3 CullGridCoord = PageGridCoord / CULL_GRID_FACTOR;
		uint CullGridLinearIndex = (CullGridCoord.z * CullGridResolution.y + CullGridCoord.y) * CullGridResolution.x + CullGridCoord.x;
		uint CullGridObjectNum = CullGridObjectHeader[CullGridLinearIndex * 2 + 0];
		uint CullGridObjectOffset = CullGridObjectHeader[CullGridLinearIndex * 2 + 1];

		for (uint IndexInCullCell = ThreadIndex; IndexInCullCell < CullGridObjectNum; IndexInCullCell += THREADGROUP_SIZE_X)
		{
			uint ObjectIndex = CullGridObjectArray[CullGridObjectOffset + IndexInCullCell];

			const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
			float SquaredDistanceBetweenAABBs = ComputeSquaredDistanceBetweenAABBs(PageTranslatedWorldCenter, PageWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

			if (SquaredDistanceBetweenAABBs < ClipmapInfluenceRadius * ClipmapInfluenceRadius)
			{
				float DistanceToPageCenter = DistanceToNearestSurfaceForObject(ObjectIndex, PageWorldCenter, InfluenceMax);
				MinDistanceToPageCenter = min(MinDistanceToPageCenter, DistanceToPageCenter);
			}
		}

		GroupMinPageDistance[ThreadIndex] = MinDistanceToPageCenter;
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 32)
	{
		GroupMinPageDistance[ThreadIndex] = min(GroupMinPageDistance[ThreadIndex], GroupMinPageDistance[ThreadIndex + 32]);
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 16)
	{
		GroupMinPageDistance[ThreadIndex] = min(GroupMinPageDistance[ThreadIndex], GroupMinPageDistance[ThreadIndex + 16]);
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 8)
	{
		GroupMinPageDistance[ThreadIndex] = min(GroupMinPageDistance[ThreadIndex], GroupMinPageDistance[ThreadIndex + 8]);
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 4)
	{
		GroupMinPageDistance[ThreadIndex] = min(GroupMinPageDistance[ThreadIndex], GroupMinPageDistance[ThreadIndex + 4]);
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 2)
	{
		GroupMinPageDistance[ThreadIndex] = min(GroupMinPageDistance[ThreadIndex], GroupMinPageDistance[ThreadIndex + 2]);
	}

	GroupMemoryBarrierWithGroupSync();
#endif // PROCESS_DISTANCE_FIELDS

	if (ThreadIndex == 0)
	{
#if PROCESS_DISTANCE_FIELDS
		float MinPageDistance = min(GroupMinPageDistance[0], GroupMinPageDistance[1]);

		if (MinPageDistance + PageWorldRadius >= -ClipmapInfluenceRadius && MinPageDistance - PageWorldRadius <= ClipmapInfluenceRadius)
		{
			bAnyObjectsInPage = true;
		}
#endif // PROCESS_DISTANCE_FIELDS

		FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(RWPageTableLayerTexture[PageTableTextureCoord]);

		if (bAnyObjectsInPage)
		{
			// Allocate new page if required
			if (!Page.bValid)
			{
				int FreeListSize = 0;
				InterlockedAdd(RWPageFreeListAllocatorBuffer[0], -1, FreeListSize);

				uint AllocatedPageId = GLOBAL_DISTANCE_FIELD_INVALID_PAGE_ID;
				if (FreeListSize > 0)
				{
					// Grab last element
					AllocatedPageId = PageFreeListBuffer[FreeListSize - 1];
				}

				if (AllocatedPageId != GLOBAL_DISTANCE_FIELD_INVALID_PAGE_ID)
				{
					Page.PageIndex = AllocatedPageId;
					Page.bCoverage = false;
					Page.bValid = true;
					RWPageTableLayerTexture[PageTableTextureCoord] = PackGlobalDistanceFieldPage(Page);
				}
			}
		}
		else
		{
			if (Page.bValid)
			{
				// Delete current page and return it to the free list

				// Return to the free list
				uint FreeListReturnIndex = 0;
				InterlockedAdd(RWPageFreeListReturnAllocatorBuffer[0], 1, FreeListReturnIndex);
				RWPageFreeListReturnBuffer[FreeListReturnIndex] = Page.PageIndex;

				Page.bValid = false;
				RWPageTableLayerTexture[PageTableTextureCoord] = PackGlobalDistanceFieldPage(Page);
			}
		}

		// Update combined (static and movable) page table
		FGlobalDistanceFieldPage CombinedPage = Page;
		#if COMPOSE_PARENT_DISTANCE_FIELD
		{
			if (!CombinedPage.bValid)
			{
				CombinedPage = UnpackGlobalDistanceFieldPage(ParentPageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));
			}
			RWPageTableCombinedTexture[PageTableTextureCoord] = PackGlobalDistanceFieldPage(CombinedPage);
		}
		#endif

		// Add new dispatch tile for this page if compose is required for it
		if (Page.bValid)
		{
			uint DestIndex;
			InterlockedAdd(RWPageComposeIndirectArgBuffer[0], GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM, DestIndex);
			RWPageComposeTileBuffer[DestIndex / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM] = PackPageUpdateTile(PageGridCoord);

			InterlockedAdd(RWBuildObjectGridIndirectArgBuffer[0], 1);
		}
	}
}

#endif

StructuredBuffer<uint> PageFreeListReturnAllocatorBuffer;
StructuredBuffer<uint> PageFreeListReturnBuffer;
RWBuffer<uint> RWFreeListReturnIndirectArgBuffer;

#ifdef PageFreeListReturnIndirectArgBufferCS

[numthreads(1, 1, 1)]
void PageFreeListReturnIndirectArgBufferCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (DispatchThreadId.x == 0)
	{
		WriteDispatchIndirectArgs(RWFreeListReturnIndirectArgBuffer, 0, (PageFreeListReturnAllocatorBuffer[0] + 63) / 64, 1, 1);

		// Clamp to 0
		RWPageFreeListAllocatorBuffer[0] = max(RWPageFreeListAllocatorBuffer[0], 0);
	}
}

#endif

#ifdef PageFreeListReturnCS

[numthreads(THREADGROUP_SIZE_X, 1, 1)]
void PageFreeListReturnCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint IndexInReturnList = DispatchThreadId.x;

	if (IndexInReturnList < PageFreeListReturnAllocatorBuffer[0])
	{
		uint FreeListLastElementIndex = 0;
		InterlockedAdd(RWPageFreeListAllocatorBuffer[0], 1, FreeListLastElementIndex);

		RWPageFreeListBuffer[FreeListLastElementIndex] = PageFreeListReturnBuffer[IndexInReturnList];
	}
}

#endif

#ifdef CullObjectsToGridCS

groupshared uint NumObjectsGridCell;
groupshared uint GridCellStartOffset;

StructuredBuffer<uint> ObjectIndexBuffer;
StructuredBuffer<uint> ObjectIndexNumBuffer;
StructuredBuffer<uint> CullGridTileBuffer;
float3 CullTileWorldExtent;
uint PackedClipmapIndex;

[numthreads(GRID_CULL_THREADGROUP_TOTALSIZE, 1, 1)]
void CullObjectsToGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint ThreadIndex = GroupThreadId.x;

	uint3 CullGridCoord = UnpackPageUpdateTile(CullGridTileBuffer[GroupId.x]);

	uint CullGridLinearIndex = (CullGridCoord.z * CullGridResolution.y + CullGridCoord.y) * CullGridResolution.x + CullGridCoord.x;

	if (ThreadIndex == 0)
	{
		NumObjectsGridCell = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	float3 CullTileTranslatedWorldCenter = CullGridCoord * CullGridCoordToTranslatedWorldCenterScale + CullGridCoordToTranslatedWorldCenterBias;

	if (all(CullGridCoord < CullGridResolution))
	{
		for (uint IndexInObjectIndexBuffer = ThreadIndex; IndexInObjectIndexBuffer < ObjectIndexNumBuffer[PackedClipmapIndex]; IndexInObjectIndexBuffer += GRID_CULL_THREADGROUP_TOTALSIZE)
		{
			uint ObjectIndex = ObjectIndexBuffer[PackedClipmapIndex * ObjectIndexBufferStride + IndexInObjectIndexBuffer];

			const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(CullTileTranslatedWorldCenter, CullTileWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

			if (DistanceSq <= InfluenceRadiusSq)
			{
				InterlockedAdd(NumObjectsGridCell, 1U);
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex == 0)
	{
		// Allocate space in the array for our objects
		InterlockedAdd(RWCullGridAllocator[0], NumObjectsGridCell, GridCellStartOffset);
		RWCullGridObjectHeader[CullGridLinearIndex * 2 + 0] = NumObjectsGridCell;
		RWCullGridObjectHeader[CullGridLinearIndex * 2 + 1] = GridCellStartOffset;
		NumObjectsGridCell = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	if (all(CullGridCoord < CullGridResolution))
	{
		for (uint IndexInObjectIndexBuffer = ThreadIndex; IndexInObjectIndexBuffer < ObjectIndexNumBuffer[PackedClipmapIndex]; IndexInObjectIndexBuffer += GRID_CULL_THREADGROUP_TOTALSIZE)
		{
			uint ObjectIndex = ObjectIndexBuffer[PackedClipmapIndex * ObjectIndexBufferStride + IndexInObjectIndexBuffer];

			const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(CullTileTranslatedWorldCenter, CullTileWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

			if (DistanceSq <= InfluenceRadiusSq)
			{
				uint DestIndex;
				InterlockedAdd(NumObjectsGridCell, 1U, DestIndex);

				// Write the intersecting object index into the array
				RWCullGridObjectArray[GridCellStartOffset + DestIndex] = ObjectIndex;
			}
		}
	}
}

#endif
