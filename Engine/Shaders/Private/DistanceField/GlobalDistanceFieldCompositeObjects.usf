// Copyright Epic Games, Inc. All Rights Reserved.

#define USE_DISTANCE_FIELD_SAMPLER 1

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../DistanceFieldLightingShared.ush"
#include "../MeshDistanceFieldCommon.ush"
#include "GlobalDistanceFieldUpdate.ush"
#include "GlobalDistanceFieldShared.ush"
#include "GlobalDistanceFieldUtils.ush"
#include "GlobalDistanceFieldObjectGrid.ush"

// Tweaked for the LDS size / occupancy
#define MAX_CULLED_DF_OBJECTS 511u
groupshared uint SharedCulledObjectList[MAX_CULLED_DF_OBJECTS];
groupshared uint NumTileCulledObjects;

#define USE_CULL_GRID 1

RWTexture3D<UNORM float> RWPageAtlasTexture;
RWTexture3D<UNORM float> RWCoverageAtlasTexture;

Texture3D<uint> PageTableLayerTexture;
RWTexture3D<uint> RWPageTableCombinedTexture;
Texture3D<uint> ParentPageTableLayerTexture;

StructuredBuffer<uint> ComposeTileBuffer;
StructuredBuffer<uint> PageTileBuffer;
float InfluenceRadius;
float InfluenceRadiusSq;
uint3 ClipmapResolution;
float3 PageCoordToVoxelTranslatedCenterScale;
float3 PageCoordToVoxelTranslatedCenterBias;
float3 ComposeTileWorldExtent;
float3 ClipmapMinBounds;
float ClipmapVoxelExtent;

float3 PreViewTranslationHigh;
float3 PreViewTranslationLow;

StructuredBuffer<uint> CullGridObjectHeader;
StructuredBuffer<uint> CullGridObjectArray;
uint3 CullGridResolution;

void CompositeMeshSDF(uint ObjectIndex, FDFVector3 VoxelWorldCenter, inout float MinDistance, inout bool bVoxelHasOneSidedMeshes, inout bool bVoxelHasTwoSidedMeshes)
{
	float MaxEarlyOutDistance = 0.0f;

#if COMPOSITE_COVERAGE_ATLAS
	// Only allow skipping the SDF sample if we are beyond the range that is needed for COMPOSITE_COVERAGE_ATLAS
	MaxEarlyOutDistance = ClipmapVoxelExtent * ONE_SIDED_BAND_SIZE;
#endif

	float MaxQueryDistance = min(InfluenceRadius, max(MinDistance, MaxEarlyOutDistance));

	FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
	float DistanceToOccluder = DistanceToNearestSurfaceForObject(DFObjectData, VoxelWorldCenter, MaxQueryDistance);
	MinDistance = min(MinDistance, DistanceToOccluder);

#if COMPOSITE_COVERAGE_ATLAS
	if (abs(DistanceToOccluder) < ClipmapVoxelExtent * ONE_SIDED_BAND_SIZE)
	{
		if (DFObjectData.bMostlyTwoSided)
		{
			bVoxelHasTwoSidedMeshes = true;
		}
		else
		{
			bVoxelHasOneSidedMeshes = true;
		}
	}
#endif
}

groupshared uint SharedCompositeMask;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void CompositeObjectsIntoPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	if (all(GroupThreadId == 0))
	{
		SharedCompositeMask = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint ThreadIndex = (GroupThreadId.z * THREADGROUP_SIZE + GroupThreadId.y) * THREADGROUP_SIZE + GroupThreadId.x;
	uint3 PageGridCoord = UnpackPageUpdateTile(ComposeTileBuffer[GroupId.x / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM]);
	uint LinearPageGroupOffset = GroupId.x % GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM;

	uint3 PageGroupOffset;
	PageGroupOffset.x = LinearPageGroupOffset % GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X;
	PageGroupOffset.y = (LinearPageGroupOffset / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X) % GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X;
	PageGroupOffset.z = (LinearPageGroupOffset / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X) / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X;

	uint3 TexelCoordInPage = PageGroupOffset * 4 + GroupThreadId;

	float3 PageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + TexelCoordInPage - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	float3 ComposeTileCenterPageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + PageGroupOffset * 4 + 1.5f - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	uint3 CullGridCoord = PageGridCoord / CULL_GRID_FACTOR;

	float3 VoxelTranslatedWorldCenter = PageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	float3 ComposeTileTranslatedWorldCenter = ComposeTileCenterPageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	const FDFVector3 VoxelWorldCenter = DFFastSubtract(VoxelTranslatedWorldCenter, PreViewTranslation);

	uint3 PageTableTextureCoord = PageGridCoordToPageTableTextureCoord(PageGridCoord);
	FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(RWPageTableCombinedTexture[PageTableTextureCoord]);
	uint3 PageAtlasCoord = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page) * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS + TexelCoordInPage;

	uint CullGridLinearIndex = (CullGridCoord.z * CullGridResolution.y + CullGridCoord.y) * CullGridResolution.x + CullGridCoord.x;
	uint CullGridObjectNum = CullGridObjectHeader[CullGridLinearIndex * 2 + 0];
	uint CullGridObjectOffset = CullGridObjectHeader[CullGridLinearIndex * 2 + 1];

	float MinDistance = InfluenceRadius;
	bool bVoxelHasOneSidedMeshes = false;
	bool bVoxelHasTwoSidedMeshes = false;

#if PROCESS_DISTANCE_FIELDS
#undef USE_OBJECT_COMPOSITING_TILE_CULLING
#define USE_OBJECT_COMPOSITING_TILE_CULLING 1
#if USE_OBJECT_COMPOSITING_TILE_CULLING

	uint NumPasses = (CullGridObjectNum + MAX_CULLED_DF_OBJECTS - 1) / MAX_CULLED_DF_OBJECTS;

	for (uint PassIndex = 0; PassIndex < NumPasses; PassIndex++)
	{
		uint PassStartObject = PassIndex * MAX_CULLED_DF_OBJECTS;
		uint PassNumObjects = PassStartObject + min(CullGridObjectNum - PassStartObject, MAX_CULLED_DF_OBJECTS);

		if (all(GroupThreadId == 0))
		{
			NumTileCulledObjects = 0;
		}

		GroupMemoryBarrierWithGroupSync();

		for (uint IndexInCullCell = PassStartObject + ThreadIndex; IndexInCullCell < PassNumObjects; IndexInCullCell += THREADGROUP_SIZE * THREADGROUP_SIZE * THREADGROUP_SIZE)
		{
			uint ObjectIndex = CullGridObjectArray[CullGridObjectOffset + IndexInCullCell];

			const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(ComposeTileTranslatedWorldCenter, ComposeTileWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

			if (DistanceSq < InfluenceRadiusSq)
			{
				uint DestIndex;
				InterlockedAdd(NumTileCulledObjects, 1U, DestIndex);
				SharedCulledObjectList[DestIndex] = ObjectIndex;
			}
		}

		GroupMemoryBarrierWithGroupSync();

		uint LocalNumTileCulledObjects = NumTileCulledObjects;

		LOOP 
		for (uint ListObjectIndex = 0; ListObjectIndex < LocalNumTileCulledObjects; ListObjectIndex++)
		{
			uint ObjectIndex = SharedCulledObjectList[ListObjectIndex];
			CompositeMeshSDF(ObjectIndex, VoxelWorldCenter, MinDistance, bVoxelHasOneSidedMeshes, bVoxelHasTwoSidedMeshes);
		}
	}

#else

	for (uint IndexInCullGrid = 0; IndexInCullGrid < CullGridObjectNum; ++IndexInCullGrid)
	{
		uint ObjectIndex = CullGridObjectArray[CullGridObjectOffset + IndexInCullGrid];
		CompositeMeshSDF(ObjectIndex, VoxelWorldCenter, MinDistance, bVoxelHasOneSidedMeshes, bVoxelHasTwoSidedMeshes);
	}

#endif
#endif // PROCESS_DISTANCE_FIELDS

	GroupMemoryBarrierWithGroupSync();

	if (all(TexelCoordInPage < GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS))
	{
		uint3 CoverageTexelCoordInPage = TexelCoordInPage / GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR;
		bool bThreadWritesCoverage = all(and(
			TexelCoordInPage % GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR == 0,
			CoverageTexelCoordInPage < GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS));

		#if COMPOSE_PARENT_DISTANCE_FIELD
		{
			FGlobalDistanceFieldPage ParentPage = UnpackGlobalDistanceFieldPage(ParentPageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));
			if (ParentPage.bValid)
			{
				uint3 ParentPageAtlasOffset = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(ParentPage);
				uint3 ParentPageAtlasCoord = ParentPageAtlasOffset * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS + TexelCoordInPage;
				float ParentDistanceField = DecodeGlobalDistanceFieldPageDistance(RWPageAtlasTexture[ParentPageAtlasCoord].x, InfluenceRadius);

				MinDistance = min(MinDistance, ParentDistanceField);

				#if COMPOSITE_COVERAGE_ATLAS
				if (bThreadWritesCoverage)
				{
					uint3 ParentCoverageAtlasCoord = ParentPageAtlasOffset * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS + CoverageTexelCoordInPage;

					// Composite parent coverage
					if (abs(ParentDistanceField) < ClipmapVoxelExtent * ONE_SIDED_BAND_SIZE)
					{
						float ParentCoverage = RWCoverageAtlasTexture[ParentCoverageAtlasCoord];
						if (ParentCoverage < 0.5f)
						{
							bVoxelHasTwoSidedMeshes = true;
						}
						else
						{
							bVoxelHasOneSidedMeshes = true;
						}
					}
				}
				#endif
			}
		}
		#endif

		RWPageAtlasTexture[PageAtlasCoord] = EncodeGlobalDistanceFieldPageDistance(MinDistance, InfluenceRadius);

		#if COMPOSITE_COVERAGE_ATLAS
		{
			if (bThreadWritesCoverage)
			{
				// Voxel containing only two sided meshes in all cache layers will be marked as 0
				float Coverage = 1.0f;
				if (bVoxelHasTwoSidedMeshes && !bVoxelHasOneSidedMeshes)
				{
					InterlockedOr(SharedCompositeMask, GLOBAL_DISTANCE_FIELD_PAGE_COVERAGE_BIT);
					Coverage = 0.0f;
				}

				uint3 CoverageAtlasCoord = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page) * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS + CoverageTexelCoordInPage;
				RWCoverageAtlasTexture[CoverageAtlasCoord] = Coverage;
			}
		}
		#endif

	}

#if COMPOSITE_COVERAGE_ATLAS
	GroupMemoryBarrierWithGroupSync();

	// Mark page containing coverage values below 1
	if (all(GroupThreadId == 0) && SharedCompositeMask != 0)
	{
		InterlockedOr(RWPageTableCombinedTexture[PageTableTextureCoord], SharedCompositeMask);
	}
#endif
}

RWStructuredBuffer<uint4> RWPageObjectGridBuffer;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void CompositeObjectsIntoObjectGridPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint ThreadIndex = (GroupThreadId.z * THREADGROUP_SIZE + GroupThreadId.y) * THREADGROUP_SIZE + GroupThreadId.x;

	// One group per page
	uint3 PageGridCoord = UnpackPageUpdateTile(ComposeTileBuffer[GroupId.x]);

	// One thread per cell in a page
	uint3 CellCoordInPage = GroupThreadId.xyz;
	uint CellOffsetInPage = ZOrder3DEncode(CellCoordInPage, log2(DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION));

	uint3 PageTableTextureCoord = PageGridCoordToPageTableTextureCoord(PageGridCoord);
	FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(PageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));

	uint3 CullGridCoord = PageGridCoord / CULL_GRID_FACTOR;
	uint CullGridLinearIndex = (CullGridCoord.z * CullGridResolution.y + CullGridCoord.y) * CullGridResolution.x + CullGridCoord.x;
	uint CullGridObjectNum = CullGridObjectHeader[CullGridLinearIndex * 2 + 0];
	uint CullGridObjectOffset = CullGridObjectHeader[CullGridLinearIndex * 2 + 1];

	float3 ObjectGridCellPageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + CellCoordInPage * 2.0f + 0.5f - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	float3 ObjectGridCellTranslatedWorldCenter = ObjectGridCellPageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	FDFVector3 ObjectGridCellWorldCenter = DFFastSubtract(ObjectGridCellTranslatedWorldCenter, PreViewTranslation);
	float ObjectGridCellWorldExtent = ClipmapVoxelExtent * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION / float(DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION);
	float CardInterpolationRange = DISTANCE_FIELD_OBJECT_GRID_CARD_INTERPOLATION_RANGE_IN_VOXELS * ClipmapVoxelExtent;
	float MaxQueryDistance = 1.44f * ObjectGridCellWorldExtent + CardInterpolationRange;

	FObjectGridCell ObjectGridCell = InitObjectGridCell();

	#if COMPOSE_PARENT_DISTANCE_FIELD
	{
		// Load parent data in order to composite movable and static cache layers
		FGlobalDistanceFieldPage ParentPage = UnpackGlobalDistanceFieldPage(ParentPageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));
		if (ParentPage.bValid)
		{
			ObjectGridCell.PackedIndex4 = RWPageObjectGridBuffer[DISTANCE_FIELD_OBJECT_GRID_PAGE_STRIDE * ParentPage.PageIndex + CellOffsetInPage];
		}
	}
	#endif

#if PROCESS_DISTANCE_FIELDS
	uint NumPasses = (CullGridObjectNum + MAX_CULLED_DF_OBJECTS - 1) / MAX_CULLED_DF_OBJECTS;
	for (uint PassIndex = 0; PassIndex < NumPasses; PassIndex++)
	{
		uint PassStartObject = PassIndex * MAX_CULLED_DF_OBJECTS;
		uint PassNumObjects = PassStartObject + min(CullGridObjectNum - PassStartObject, MAX_CULLED_DF_OBJECTS);

		if (all(GroupThreadId == 0))
		{
			NumTileCulledObjects = 0;
		}

		GroupMemoryBarrierWithGroupSync();

		// Cull to tiles
		{
			float3 ObjectGridTilePageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + 2.0f * (GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION / 4.0f);
			float3 ObjectGridTileTranslatedWorldCenter = ObjectGridTilePageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
			float ObjectGridTileWorldExtent = ClipmapVoxelExtent * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION;
			float MaxTileQueryDistance = 1.44f * ObjectGridTileWorldExtent + CardInterpolationRange;

			for (uint IndexInCullCell = PassStartObject + ThreadIndex; IndexInCullCell < PassNumObjects; IndexInCullCell += THREADGROUP_SIZE * THREADGROUP_SIZE * THREADGROUP_SIZE)
			{
				uint ObjectIndex = CullGridObjectArray[CullGridObjectOffset + IndexInCullCell];

				FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
				const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
				float DistanceSq = ComputeSquaredDistanceBetweenAABBs(ObjectGridTileTranslatedWorldCenter, ObjectGridTileWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

				if (DistanceSq < Pow2(MaxTileQueryDistance))
				{
					uint DestIndex;
					InterlockedAdd(NumTileCulledObjects, 1U, DestIndex);
					SharedCulledObjectList[DestIndex] = ObjectIndex;
				}
			}
		}

		GroupMemoryBarrierWithGroupSync();

		uint LocalNumTileCulledObjects = NumTileCulledObjects;
		for (uint ListObjectIndex = 0; ListObjectIndex < LocalNumTileCulledObjects; ListObjectIndex++)
		{
			uint ObjectIndex = SharedCulledObjectList[ListObjectIndex];

			FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PreViewTranslation);
			float DistanceSq = ComputeSquaredDistanceBetweenAABBs(ObjectGridCellTranslatedWorldCenter, ObjectGridCellWorldExtent, TranslatedBoundsCenter, DFObjectBounds.BoxExtent);

			if (DistanceSq <= Pow2(CardInterpolationRange))
			{
				FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
				float DistanceToOccluder = DistanceToNearestSurfaceForObject(DFObjectData, ObjectGridCellWorldCenter, MaxQueryDistance);
				AddToObjectGridCell(ObjectGridCell, DFObjectData.GPUSceneInstanceIndex, DistanceToOccluder, ObjectGridCellWorldExtent, MaxQueryDistance);
			}
		}
	}
#endif

	SortObjectGridCell(ObjectGridCell);
	RWPageObjectGridBuffer[DISTANCE_FIELD_OBJECT_GRID_PAGE_STRIDE * Page.PageIndex + CellOffsetInPage] = ObjectGridCell.PackedIndex4;
}