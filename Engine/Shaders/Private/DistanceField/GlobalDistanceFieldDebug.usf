// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShaderPrint.ush"
#include "GlobalDistanceFieldShared.ush"
#include "GlobalDistanceFieldUpdate.ush"

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 0
#endif

StructuredBuffer<uint> GlobalDistanceFieldPageFreeListAllocatorBuffer;
uint GlobalDistanceFieldMaxPageNum;

uint ClipmapSizeInPages;
uint ClipmapIndex;
float3 InvPageAtlasSize;
RWStructuredBuffer<uint> RWPageStatsBuffer;
StructuredBuffer<uint> PageStatsBuffer;
Texture3D<uint> PageTableCombinedTexture;
Texture3D PageAtlasTexture;
SamplerState DistanceFieldSampler;

Buffer<uint> PageComposeIndirectArgBuffer;

#ifdef GlobalDistanceFieldAccumulateUpdatedPagesCS

[numthreads(1, 1, 1)]
void GlobalDistanceFieldAccumulateUpdatedPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (all(DispatchThreadId.xyz == 0))
	{
		InterlockedAdd(RWPageStatsBuffer[2], PageComposeIndirectArgBuffer[0] / GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM);
	}
}

#endif

#ifdef GlobalDistanceFieldPageStatsCS

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void GlobalDistanceFieldPageStatsCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint3 PageTableCoord = DispatchThreadId.xyz;
	if (all(PageTableCoord < ClipmapSizeInPages))
	{
		PageTableCoord.z += ClipmapIndex * ClipmapSizeInPages;

		FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(PageTableCombinedTexture.Load(uint4(PageTableCoord, 0)));
		if (Page.bValid)
		{
			InterlockedAdd(RWPageStatsBuffer[0], 1);

			bool bFilled = false;

			for (uint PageCoordZ = 0; PageCoordZ < GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION; ++PageCoordZ)
			{
				for (uint PageCoordY = 0; PageCoordY < GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION; ++PageCoordY)
				{
					for (uint PageCoordX = 0; PageCoordX < GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION; ++PageCoordX)
					{
						// Don't sample border values which can be safely skipped
						float3 CoordInPage = uint3(PageCoordX, PageCoordY, PageCoordZ);
						CoordInPage = clamp(CoordInPage + 0.5f, 1.0f, GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION - 1.0f);

						uint3 PageAtlasOffset = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page) * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS;
						float3 PageUV = (PageAtlasOffset + CoordInPage) * InvPageAtlasSize;
						float DistanceFieldValue = Texture3DSampleLevel(PageAtlasTexture, DistanceFieldSampler, PageUV, 0).x;

						if (DistanceFieldValue > 0.0f && DistanceFieldValue < 1.0f)
						{
							bFilled = true;
						}
					}
				}
			}

			if (!bFilled)
			{
				InterlockedAdd(RWPageStatsBuffer[1], 1);
			}
		}
	}
}

#endif

#ifdef GlobalDistanceFieldDebugCS

[numthreads(1, 1, 1)]
void GlobalDistanceFieldDebugCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (all(DispatchThreadId.xyz == 0))
	{
		int NumFreePages = GlobalDistanceFieldPageFreeListAllocatorBuffer[0];
		int NumAllocPages = -1;
		if (GlobalDistanceFieldMaxPageNum > 0)
		{
			NumAllocPages = GlobalDistanceFieldMaxPageNum - GlobalDistanceFieldPageFreeListAllocatorBuffer[0];
		}

		FShaderPrintContext Context = InitShaderPrintContext(true, float2(0.8, 0.2));

		Print(Context, TEXT("GDF Pages"));
		Newline(Context);

		Print(Context, TEXT(" Max   "));
		Print(Context, GlobalDistanceFieldMaxPageNum);
		Newline(Context);

		Print(Context, TEXT(" Alloc "), FontRed);
		Print(Context, NumAllocPages, FontRed);
		Newline(Context);

		Print(Context, TEXT(" Free  "), FontGreen);
		Print(Context, NumFreePages, FontGreen);
		Newline(Context);

		Print(Context, TEXT(" Combi "));
		Print(Context, PageStatsBuffer[0]);
		Newline(Context);

		Print(Context, TEXT(" Empty "));
		Print(Context, PageStatsBuffer[1]);
		Newline(Context);

		Print(Context, TEXT(" Updated  "), FontOrange);
		Print(Context, PageStatsBuffer[2], FontOrange);
		Newline(Context);
	}
}

#endif
