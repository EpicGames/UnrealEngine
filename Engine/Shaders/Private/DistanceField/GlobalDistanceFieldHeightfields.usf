// Copyright Epic Games, Inc. All Rights Reserved.

#define USE_DISTANCE_FIELD_SAMPLER 1

#include "../Common.ush"
#include "../HeightfieldLightingShared.ush"
#include "GlobalDistanceFieldUpdate.ush"
#include "GlobalDistanceFieldShared.ush"
#include "GlobalDistanceFieldUtils.ush"
#include "GlobalDistanceFieldObjectGrid.ush"

RWStructuredBuffer<uint> RWMarkedHeightfieldPageBuffer;
RWTexture3D<UNORM float> RWPageAtlasTexture;
RWTexture3D<UNORM float> RWCoverageAtlasTexture;

StructuredBuffer<uint> MarkedHeightfieldPageBuffer;
StructuredBuffer<uint> PageUpdateTileBuffer;
StructuredBuffer<uint> ComposeTileBuffer;
Texture3D<uint> PageTableLayerTexture;

float3 PageCoordToVoxelTranslatedCenterScale;
float3 PageCoordToVoxelTranslatedCenterBias;
float3 PageWorldExtent;
float3 InvPageGridResolution;
float ClipmapVoxelExtent;
float InfluenceRadius;
float HeightfieldThickness;
float3 PreViewTranslationHigh;
float3 PreViewTranslationLow;

struct FHeightfieldSample
{
	bool bValid;
	uint HeightfieldIndex;
	float3 TranslatedWorldPosition;
	float3 WorldNormal;
};

FHeightfieldSample SampleHeightfield(FDFVector3 PreViewTranslation, FDFVector3 VoxelWorldCenter, float VoxelWorldExtent)
{
	FHeightfieldSample Sample;
	Sample.bValid = false;
	Sample.HeightfieldIndex = NumHeightfields;
	Sample.TranslatedWorldPosition = float3(0.0f, 0.0f, 0.0f);
	Sample.WorldNormal = float3(0.0f, 0.0f, 0.0f);

	for (uint HeightfieldIndex = 0; HeightfieldIndex < NumHeightfields; HeightfieldIndex++)
	{
		float3 LocalPosition = DFMultiplyAffineDemote(VoxelWorldCenter, GetWorldToLocal(HeightfieldIndex));
		float4 MinMaxHeightfieldUV;
		float2 HeightfieldUV = GetHeightfieldUV(HeightfieldIndex, LocalPosition.xy, MinMaxHeightfieldUV);

		if (all(HeightfieldUV >= MinMaxHeightfieldUV.xy) && all(HeightfieldUV <= MinMaxHeightfieldUV.zw))
		{
			Sample.HeightfieldIndex = HeightfieldIndex;
			break;
		}
	}

	if (Sample.HeightfieldIndex < NumHeightfields)
	{
		float3 LocalPosition = DFMultiplyAffineDemote(VoxelWorldCenter, GetWorldToLocal(Sample.HeightfieldIndex));
		float4 MinMaxHeightfieldUV;
		float2 HeightfieldUV = GetHeightfieldUV(Sample.HeightfieldIndex, LocalPosition.xy, MinMaxHeightfieldUV);

		if (all(HeightfieldUV >= MinMaxHeightfieldUV.xy) && all(HeightfieldUV <= MinMaxHeightfieldUV.zw))
		{
			float HeightfieldVisibility;
			FDFVector3 WorldHeightfieldShadingPosition = GetHeightfieldWorldPositionAndNormal(Sample.HeightfieldIndex, LocalPosition.xy, HeightfieldUV, VoxelWorldExtent, Sample.WorldNormal, HeightfieldVisibility);
			Sample.TranslatedWorldPosition = DFFastToTranslatedWorld(WorldHeightfieldShadingPosition, PreViewTranslation);

			// Skip holes in the heightfield
			if (HeightfieldVisibility > 0.5f)
			{
				Sample.bValid = true;
			}
		}
	}

	return Sample;
}

float ComputeHeightfieldDistance(FHeightfieldSample HeightfieldSample, float3 TranslatedWorldPosition)
{
	// Project the vertical height vector onto the normal of the heightfield directly below the point we are computing the distance field for, use the perpendicular distance
	float DistanceToHeightfieldPlane = dot(HeightfieldSample.WorldNormal, TranslatedWorldPosition - HeightfieldSample.TranslatedWorldPosition);

	// Limit negative region of a heightfield to a user defined thickness  
	const float MinInteriorDistance = -HeightfieldThickness;
	if (DistanceToHeightfieldPlane < MinInteriorDistance)
	{
		DistanceToHeightfieldPlane = MinInteriorDistance - DistanceToHeightfieldPlane;
	}

	float HeightfieldMinDistance = clamp(DistanceToHeightfieldPlane, -InfluenceRadius, InfluenceRadius);
	return HeightfieldMinDistance;
}

#ifdef MarkHeightfieldPagesCS

groupshared uint GroupMarkedPage[64];

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void MarkHeightfieldPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint ThreadIndex = GroupThreadId.x + GroupThreadId.y * THREADGROUP_SIZE;
	uint IndexInPageBuffer = GroupId.x;

	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint3 TexelCoordInPage = uint3(GroupThreadId.xy, 0);
	uint3 PageGridCoord = UnpackPageUpdateTile(PageUpdateTileBuffer[IndexInPageBuffer]);
	float3 PageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + TexelCoordInPage - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	float3 VoxelTranslatedWorldCenter = PageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	FDFVector3 VoxelWorldCenter = DFFastSubtract(VoxelTranslatedWorldCenter, PreViewTranslation);

	GroupMarkedPage[ThreadIndex] = 0;

	FHeightfieldSample HeightfieldSample = SampleHeightfield(PreViewTranslation, VoxelWorldCenter, ClipmapVoxelExtent);
	if (HeightfieldSample.bValid)
	{
		float3 VoxelTranslatedWorldCenterNearestZ = VoxelTranslatedWorldCenter;

		float MinZ = VoxelTranslatedWorldCenter.z + GLOBAL_DISTANCE_FIELD_PAGE_BORDER * ClipmapVoxelExtent;
		float MaxZ = VoxelTranslatedWorldCenter.z + (GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS - GLOBAL_DISTANCE_FIELD_PAGE_BORDER) * ClipmapVoxelExtent;
		VoxelTranslatedWorldCenterNearestZ.z = clamp(HeightfieldSample.TranslatedWorldPosition.z, MinZ, MaxZ);

		float HeightfieldDistance = ComputeHeightfieldDistance(HeightfieldSample, VoxelTranslatedWorldCenterNearestZ);

		if (abs(HeightfieldDistance) < InfluenceRadius)
		{
			GroupMarkedPage[ThreadIndex] = 1;
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 32)
	{
		GroupMarkedPage[ThreadIndex] = GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 32];
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 16)
	{
		GroupMarkedPage[ThreadIndex] = GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 16];
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 8)
	{
		GroupMarkedPage[ThreadIndex] = GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 8];
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 4)
	{
		GroupMarkedPage[ThreadIndex] = GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 4];
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex < 2)
	{
		GroupMarkedPage[ThreadIndex] = GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 2];
	}

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex == 0)
	{
		if (GroupMarkedPage[ThreadIndex] + GroupMarkedPage[ThreadIndex + 1] > 0)
		{
			RWMarkedHeightfieldPageBuffer[IndexInPageBuffer] = 1;
		}
	}
}

#endif

RWBuffer<uint> RWBuildHeightfieldComposeTilesIndirectArgBuffer;
RWBuffer<uint> RWPageComposeHeightfieldIndirectArgBuffer;
Buffer<uint> PageUpdateIndirectArgBuffer;

#ifdef BuildHeightfieldComposeTilesIndirectArgBufferCS

[numthreads(1, 1, 1)]
void BuildHeightfieldComposeTilesIndirectArgBufferCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (DispatchThreadId.x == 0)
	{
		const uint TileNum = PageUpdateIndirectArgBuffer[0];

		WriteDispatchIndirectArgs(RWBuildHeightfieldComposeTilesIndirectArgBuffer, 0, (TileNum + 63) / 64, 1, 1);
		WriteDispatchIndirectArgs(RWPageComposeHeightfieldIndirectArgBuffer, 0, 0, 1, 1);
	}
}

#endif


#ifdef BuildHeightfieldComposeTilesCS

RWStructuredBuffer<uint> RWPageComposeHeightfieldTileBuffer;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void BuildHeightfieldComposeTilesCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const uint TileIndex = DispatchThreadId.x;
	const uint TileNum = PageUpdateIndirectArgBuffer[0];

	if (TileIndex < TileNum)
	{
		uint PackedPageGridCoord = PageUpdateTileBuffer[TileIndex];
		bool bMarkedHeightfieldPage = MarkedHeightfieldPageBuffer[TileIndex] > 0;

		if (bMarkedHeightfieldPage)
		{
			uint DestIndex;
			InterlockedAdd(RWPageComposeHeightfieldIndirectArgBuffer[0], 1, DestIndex);
			RWPageComposeHeightfieldTileBuffer[DestIndex] = PackedPageGridCoord;
		}
	}
}

#endif

#ifdef ComposeHeightfieldsIntoPagesCS

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void ComposeHeightfieldsIntoPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	uint3 TexelCoordInPage = uint3(GroupThreadId.xy, 0);
	uint3 PageGridCoord = UnpackPageUpdateTile(ComposeTileBuffer[GroupId.x]);
	float3 PageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + TexelCoordInPage - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	float3 VoxelTranslatedWorldCenter = PageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	FDFVector3 VoxelWorldCenter = DFFastSubtract(VoxelTranslatedWorldCenter, PreViewTranslation);

	uint3 PageTableTextureCoord = PageGridCoordToPageTableTextureCoord(PageGridCoord);
	FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(PageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));
	uint3 PageAtlasCoord = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page) * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS;
	PageAtlasCoord += TexelCoordInPage;

	FHeightfieldSample HeightfieldSample = SampleHeightfield(PreViewTranslation, VoxelWorldCenter, ClipmapVoxelExtent);
	if (HeightfieldSample.bValid)
	{
		// Compute distance for all Z values of the update region
		for (uint ZIndex = 0; ZIndex < GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS; ++ZIndex)
		{
			float3 TranslatedWorldPosition = VoxelTranslatedWorldCenter.xyz + float3(0.0f, 0.0f, ZIndex * PageCoordToVoxelTranslatedCenterScale.z);
			float HeightfieldDistance = ComputeHeightfieldDistance(HeightfieldSample, TranslatedWorldPosition);

			const uint3 PageAtlasCoordZ = PageAtlasCoord + uint3(0, 0, ZIndex);

			float PreviousDistanceField = DecodeGlobalDistanceFieldPageDistance(RWPageAtlasTexture[PageAtlasCoordZ], InfluenceRadius);
			float MinDistance = min(HeightfieldDistance, PreviousDistanceField);

			RWPageAtlasTexture[PageAtlasCoordZ] = EncodeGlobalDistanceFieldPageDistance(MinDistance, InfluenceRadius);

			#if COMPOSITE_COVERAGE_ATLAS
			{
				uint3 TexelCoordInPageWithZ = uint3(TexelCoordInPage.xy, ZIndex);
				bool bThreadWritesCoverage = all(TexelCoordInPageWithZ % GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR == 0);
				uint3 CoverageTexelCoordInPage = TexelCoordInPageWithZ / GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR;

				if (bThreadWritesCoverage && abs(HeightfieldDistance) < ClipmapVoxelExtent * ONE_SIDED_BAND_SIZE
					&& all(CoverageTexelCoordInPage < GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS))
				{
					uint3 CoverageAtlasCoord = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page) * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS + CoverageTexelCoordInPage;
					RWCoverageAtlasTexture[CoverageAtlasCoord] = 1.0f;
				}
			}
			#endif
		}
	}
}

#endif

#ifdef CompositeHeightfieldsIntoObjectGridPagesCS

RWStructuredBuffer<uint4> RWPageObjectGridBuffer;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void CompositeHeightfieldsIntoObjectGridPagesCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	// One thread per cell in a page
	uint3 CellCoordInPage = GroupThreadId.xyz;
	uint CellOffsetInPage = ZOrder3DEncode(CellCoordInPage, log2(DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION));

	uint3 PageGridCoord = UnpackPageUpdateTile(ComposeTileBuffer[GroupId.x]);
	uint3 PageTableTextureCoord = PageGridCoordToPageTableTextureCoord(PageGridCoord);
	FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(PageTableLayerTexture.Load(int4(PageTableTextureCoord, 0)));

	float3 ObjectGridCellPageCoord = PageGridCoord * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + CellCoordInPage * 2.0f + 0.5f - GLOBAL_DISTANCE_FIELD_PAGE_BORDER;
	float3 ObjectGridCellTranslatedWorldCenter = ObjectGridCellPageCoord * PageCoordToVoxelTranslatedCenterScale + PageCoordToVoxelTranslatedCenterBias;
	FDFVector3 ObjectGridCellWorldCenter = DFFastSubtract(ObjectGridCellTranslatedWorldCenter, PreViewTranslation);
	float ObjectGridCellWorldExtent = ClipmapVoxelExtent * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION / float(DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION);
	float CardInterpolationRange = DISTANCE_FIELD_OBJECT_GRID_CARD_INTERPOLATION_RANGE_IN_VOXELS * ClipmapVoxelExtent;
	float MaxQueryDistance = 1.44f * ObjectGridCellWorldExtent + CardInterpolationRange;

	FHeightfieldSample HeightfieldSample = SampleHeightfield(PreViewTranslation, ObjectGridCellWorldCenter, ObjectGridCellWorldExtent);
	if (HeightfieldSample.bValid)
	{
		float HeightfieldDistance = ComputeHeightfieldDistance(HeightfieldSample, ObjectGridCellTranslatedWorldCenter);
		if (HeightfieldDistance < MaxQueryDistance)
		{
			FObjectGridCell GridCell = InitObjectGridCell();
			GridCell.PackedIndex4 = RWPageObjectGridBuffer[DISTANCE_FIELD_OBJECT_GRID_PAGE_STRIDE * Page.PageIndex + CellOffsetInPage];
			AddToObjectGridCell(GridCell, GetHeightfieldGPUSceneInstanceIndex(HeightfieldSample.HeightfieldIndex), HeightfieldDistance, ObjectGridCellWorldExtent, MaxQueryDistance);
			SortObjectGridCell(GridCell);
			RWPageObjectGridBuffer[DISTANCE_FIELD_OBJECT_GRID_PAGE_STRIDE * Page.PageIndex + CellOffsetInPage] = GridCell.PackedIndex4;
		}
	}
}

#endif
