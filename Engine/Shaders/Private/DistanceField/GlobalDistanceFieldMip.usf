// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "GlobalDistanceFieldShared.ush"

RWTexture3D<float> RWMipTexture;
uint ClipmapMipResolution;
float OneOverClipmapMipResolution;
float MipInfluenceRadius;
Texture3D<uint> PageTableTexture;
Texture3D PageAtlasTexture;
Texture3D PrevMipTexture;
SamplerState DistanceFieldSampler;

// https://en.wikipedia.org/wiki/Eikonal_equation
float Eikonal1(float X, float Step)
{
	return X + Step;
}

float Eikonal2(float X, float Y, float Step)
{
	float SumU = X + Y;
	float SumUSq = X * X + Y * Y;
	float DistanceSq = SumU * SumU - 2.0f * (SumUSq - Step * Step);
	return (1.0f / 2.0f) * (SumU + sqrt(DistanceSq));
}

float Eikonal3(float X, float Y, float Z, float Step)
{
	float SumU = X + Y + Z;
	float SumUSq = X * X + Y * Y + Z * Z;
	float DistanceSq = SumU * SumU - 3.0f * (SumUSq - Step * Step);
	return (1.0f / 3.0f) * (SumU + sqrt(DistanceSq));
}

uint ClipmapIndex;
uint PrevClipmapOffsetZ;
uint ClipmapOffsetZ;
float3 ClipmapUVScrollOffset;
float CoarseDistanceFieldValueScale;
float CoarseDistanceFieldValueBias;

float LoadPrevDistanceFieldValue(int3 MipCoord, int3 Offset)
{
	MipCoord = clamp(MipCoord + Offset, 0, (int)ClipmapMipResolution - 1);

#if READ_PAGES
	// Reverse clipmap scroll offset as coarse clipmap is always centered around camera
	float3 ScrolledClipmapUV = (MipCoord + 0.5f) * OneOverClipmapMipResolution;
	ScrolledClipmapUV = frac(ScrolledClipmapUV + ClipmapUVScrollOffset); // wraparound addressing
	ScrolledClipmapUV = frac(ScrolledClipmapUV); // apply frac twice to prevent UV == 1.0f because frac(-0.00...001f) = 1.0f

	int3 PageTableTextureCoord = saturate(ScrolledClipmapUV) * GlobalDistanceFieldClipmapSizeInPages + int3(0, 0, ClipmapIndex * GlobalDistanceFieldClipmapSizeInPages);
	FGlobalDistanceFieldPage Page = UnpackGlobalDistanceFieldPage(PageTableTexture.Load(int4(PageTableTextureCoord, 0)));

	float DistanceFieldValue = 1.0f;
	if (Page.bValid)
	{
		float3 PageUV = ComputeGlobalDistanceFieldPageUV(ScrolledClipmapUV, Page);
		DistanceFieldValue = Texture3DSampleLevel(PageAtlasTexture, DistanceFieldSampler, PageUV, 0).x;
		if (DistanceFieldValue < 1.0f)
		{
			DistanceFieldValue = DistanceFieldValue * CoarseDistanceFieldValueScale + CoarseDistanceFieldValueBias;
		}
	}

	return DistanceFieldValue;
#else

	MipCoord.z += PrevClipmapOffsetZ;
	return PrevMipTexture[MipCoord].x;

#endif
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void PropagateMipDistanceCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint3 MipCoord = DispatchThreadId.xyz;

	if (all(MipCoord < ClipmapMipResolution))
	{
		float Center = LoadPrevDistanceFieldValue(MipCoord, int3(0, 0, 0));

		float3 V;
		V.x = min(LoadPrevDistanceFieldValue(MipCoord, int3(-1, 0, 0)), LoadPrevDistanceFieldValue(MipCoord, int3(+1, 0, 0)));
		V.y = min(LoadPrevDistanceFieldValue(MipCoord, int3(0, -1, 0)), LoadPrevDistanceFieldValue(MipCoord, int3(0, +1, 0)));
		V.z = min(LoadPrevDistanceFieldValue(MipCoord, int3(0, 0, -1)), LoadPrevDistanceFieldValue(MipCoord, int3(0, 0, +1)));

		float MinV = min3(V.x, V.y, V.z);
		float MaxV = max3(V.x, V.y, V.z);

		float3 SortedV;
		SortedV.x = MinV;
		SortedV.y = (V.x != MinV && V.x != MaxV) ? V.x : ((V.y != MinV && V.y != MaxV) ? V.y : V.z);
		SortedV.z = MaxV;

		const float Step = 1.0f / (2.0f * GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS);
		Center = min(Center, Eikonal1(SortedV.x, Step));
		Center = min(Center, Eikonal2(SortedV.x, SortedV.y, Step));
		Center = min(Center, Eikonal3(SortedV.x, SortedV.y, SortedV.z, Step));

		RWMipTexture[MipCoord + int3(0, 0, ClipmapOffsetZ)] = Center;
	}
}