// Copyright Epic Games, Inc. All Rights Reserved.

#define DISTANCE_FIELD_OBJECT_GRID_INVALID_ID 0xFFFFFFFF
#define DISTANCE_FIELD_OBJECT_GRID_CELL_SIZE 4 // Up to 4 objects per cell
#define DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION 4
#define DISTANCE_FIELD_OBJECT_GRID_CARD_INTERPOLATION_RANGE_IN_VOXELS 3.0f
#define DISTANCE_FIELD_OBJECT_GRID_PAGE_STRIDE (DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION * DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION * DISTANCE_FIELD_OBJECT_GRID_PAGE_RESOLUTION)

struct FObjectGridCell
{
	uint4 PackedIndex4;
};

FObjectGridCell InitObjectGridCell()
{
	FObjectGridCell GridCell;
	GridCell.PackedIndex4 = uint4(DISTANCE_FIELD_OBJECT_GRID_INVALID_ID, DISTANCE_FIELD_OBJECT_GRID_INVALID_ID, DISTANCE_FIELD_OBJECT_GRID_INVALID_ID, DISTANCE_FIELD_OBJECT_GRID_INVALID_ID);
	return GridCell;
}

struct FObjectGridCellIndex
{
	uint GPUSceneInstanceIndex;
	bool bValid;
};

uint PackObjectGridCellIndex(uint GPUSceneInstanceIndex, float Distance, float ObjectGridCellWorldExtent, float MaxQueryDistance)
{
	float NormalizedPackedDistance = saturate((Distance + 0.25f * ObjectGridCellWorldExtent) / (MaxQueryDistance + 0.25f * ObjectGridCellWorldExtent));
	uint PackedDistanceUInt = uint(clamp(NormalizedPackedDistance * (0xFF + 1.0f), 0, 0xFF));
	uint GridCellElement = (GPUSceneInstanceIndex & 0xFFFFFF) | (PackedDistanceUInt << 24);
	return GridCellElement;
}

// Unpack GPUSceneInstanceIndex
FObjectGridCellIndex UnpackObjectGridCellIndex(uint PackedIndex)
{
	FObjectGridCellIndex ObjectGridCellIndex;
	ObjectGridCellIndex.bValid = PackedIndex < DISTANCE_FIELD_OBJECT_GRID_INVALID_ID;
	ObjectGridCellIndex.GPUSceneInstanceIndex = PackedIndex & 0xFFFFFF;
	return ObjectGridCellIndex;
}

// Unordered add to a grid cell
void AddToObjectGridCell(inout FObjectGridCell ObjectGridCell, uint GPUSceneInstanceIndex, float Distance, float ObjectGridCellWorldExtent, float MaxQueryDistance)
{
	if (abs(Distance) < MaxQueryDistance)
	{
		uint PackedIndexToInsert = PackObjectGridCellIndex(GPUSceneInstanceIndex, Distance, ObjectGridCellWorldExtent, MaxQueryDistance);

		uint MaxPackedIndex = 0;
		for (uint IndexInList = 0; IndexInList < DISTANCE_FIELD_OBJECT_GRID_CELL_SIZE; ++IndexInList)
		{
			MaxPackedIndex = max(MaxPackedIndex, ObjectGridCell.PackedIndex4[IndexInList]);
		}

		if (PackedIndexToInsert < MaxPackedIndex)
		{
			for (uint IndexInList = 0; IndexInList < DISTANCE_FIELD_OBJECT_GRID_CELL_SIZE; ++IndexInList)
			{
				if (ObjectGridCell.PackedIndex4[IndexInList] == MaxPackedIndex)
				{
					ObjectGridCell.PackedIndex4[IndexInList] = PackedIndexToInsert;
					break;
				}
			}
		}
	}
}

// Sort grid cell elements by distance
void SortObjectGridCell(inout FObjectGridCell ObjectGridCell)
{
	for (uint PassIndex = 0; PassIndex < DISTANCE_FIELD_OBJECT_GRID_CELL_SIZE - 1; ++PassIndex)
	{
		for (uint ElementIndex = 0; ElementIndex < DISTANCE_FIELD_OBJECT_GRID_CELL_SIZE - PassIndex - 1; ++ElementIndex)
		{
			uint MinIndex = min(ObjectGridCell.PackedIndex4[ElementIndex], ObjectGridCell.PackedIndex4[ElementIndex + 1]);
			uint MaxIndex = max(ObjectGridCell.PackedIndex4[ElementIndex], ObjectGridCell.PackedIndex4[ElementIndex + 1]);
			ObjectGridCell.PackedIndex4[ElementIndex] = MinIndex;
			ObjectGridCell.PackedIndex4[ElementIndex + 1] = MaxIndex;
		}
	}
}