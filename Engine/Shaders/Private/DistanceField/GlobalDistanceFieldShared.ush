// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if IS_MATERIAL_SHADER
	#define DISTANCE_FIELD_IN_VIEW_UB 1
#endif

#if DISTANCE_FIELD_IN_VIEW_UB
	#define MaxGlobalDFAOConeDistance View.MaxGlobalDFAOConeDistance
#else
	float MaxGlobalDFAOConeDistance;
#endif

#if PLATFORM_SUPPORTS_DISTANCE_FIELDS

// Must match C++
// Most functions in this file still using 4 to match legacy
#define MAX_GLOBAL_DF_CLIPMAPS 6

#if DISTANCE_FIELD_IN_VIEW_UB
    // for materials, these are in the view UB
	#define GlobalDistanceFieldPageAtlasTexture				View.GlobalDistanceFieldPageAtlasTexture
	#define GlobalDistanceFieldCoverageAtlasTexture			View.GlobalDistanceFieldCoverageAtlasTexture
	#define GlobalDistanceFieldPageTableTexture				View.GlobalDistanceFieldPageTableTexture
	#define GlobalDistanceFieldMipTexture					View.GlobalDistanceFieldMipTexture
	#define GlobalVolumeTranslatedCenterAndExtent			View.GlobalVolumeTranslatedCenterAndExtent
	#define GlobalVolumeTranslatedWorldToUVAddAndMul		View.GlobalVolumeTranslatedWorldToUVAddAndMul
	#define GlobalDistanceFieldMipTranslatedWorldToUVScale	View.GlobalDistanceFieldMipTranslatedWorldToUVScale
	#define GlobalDistanceFieldMipTranslatedWorldToUVBias	View.GlobalDistanceFieldMipTranslatedWorldToUVBias
	#define GlobalDistanceFieldMipUVMinZ					View.GlobalDistanceFieldMipUVMinZ
	#define GlobalDistanceFieldMipUVMaxZ					View.GlobalDistanceFieldMipUVMaxZ
	#define GlobalDistanceFieldMipFactor					View.GlobalDistanceFieldMipFactor
	#define GlobalDistanceFieldMipTransition				View.GlobalDistanceFieldMipTransition
	#define GlobalDistanceFieldInvPageAtlasSize				View.GlobalDistanceFieldInvPageAtlasSize
	#define GlobalDistanceFieldInvCoverageAtlasSize			View.GlobalDistanceFieldInvCoverageAtlasSize
	#define GlobalDistanceFieldClipmapSizeInPages			View.GlobalDistanceFieldClipmapSizeInPages
	#define GlobalVolumeDimension							View.GlobalVolumeDimension
	#define GlobalVolumeTexelSize							View.GlobalVolumeTexelSize
	#define NumGlobalSDFClipmaps							View.NumGlobalSDFClipmaps
	#define CoveredExpandSurfaceScale						View.CoveredExpandSurfaceScale
	#define NotCoveredExpandSurfaceScale					View.NotCoveredExpandSurfaceScale
	#define NotCoveredMinStepScale							View.NotCoveredMinStepScale
	#define DitheredTransparencyStepThreshold				View.DitheredTransparencyStepThreshold
	#define DitheredTransparencyTraceThreshold				View.DitheredTransparencyTraceThreshold

#else
	// these are only used for the precomputation shaders; which don't have a view UB
	Texture3D GlobalDistanceFieldPageAtlasTexture;
	Texture3D GlobalDistanceFieldCoverageAtlasTexture;
	Texture3D<uint> GlobalDistanceFieldPageTableTexture;
	Texture3D GlobalDistanceFieldMipTexture;
	float4 GlobalVolumeTranslatedCenterAndExtent[MAX_GLOBAL_DF_CLIPMAPS];
	float4 GlobalVolumeTranslatedWorldToUVAddAndMul[MAX_GLOBAL_DF_CLIPMAPS];
	float4 GlobalDistanceFieldMipTranslatedWorldToUVScale[MAX_GLOBAL_DF_CLIPMAPS];
	float4 GlobalDistanceFieldMipTranslatedWorldToUVBias[MAX_GLOBAL_DF_CLIPMAPS];
	float GlobalDistanceFieldMipFactor;
	float GlobalDistanceFieldMipTransition;
	float3 GlobalDistanceFieldInvPageAtlasSize;
	float3 GlobalDistanceFieldInvCoverageAtlasSize;
	uint GlobalDistanceFieldClipmapSizeInPages;
	float GlobalVolumeDimension;
	float GlobalVolumeTexelSize;
	uint NumGlobalSDFClipmaps;
	float CoveredExpandSurfaceScale;
	float NotCoveredExpandSurfaceScale;
	float NotCoveredMinStepScale;
	float DitheredTransparencyStepThreshold;
	float DitheredTransparencyTraceThreshold;
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define GlobalDistanceFieldCoverageAtlasTextureSampler GlobalTrilinearWrappedSampler
	#define GlobalDistanceFieldPageAtlasTextureSampler GlobalTrilinearWrappedSampler
	#define GlobalDistanceFieldMipTextureSampler GlobalTrilinearClampedSampler
#else
	#if DISTANCE_FIELD_IN_VIEW_UB
		#define GlobalDistanceFieldCoverageAtlasTextureSampler View.GlobalDistanceFieldCoverageAtlasTextureSampler
		#define GlobalDistanceFieldPageAtlasTextureSampler View.GlobalDistanceFieldPageAtlasTextureSampler
		#define GlobalDistanceFieldMipTextureSampler View.GlobalDistanceFieldMipTextureSampler
	#else
		SamplerState GlobalDistanceFieldCoverageAtlasTextureSampler;
		SamplerState GlobalDistanceFieldPageAtlasTextureSampler;
		SamplerState GlobalDistanceFieldMipTextureSampler;
	#endif
#endif

// Must match GlobalDistanceField.cpp
#define GLOBAL_DISTANCE_FIELD_PAGE_BORDER 0.5f
#define GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS 8 // Includes 0.5 texel trilinear filter margin
#define GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION (GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS - 1)
#define GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_X 128
#define GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_Y 128
#define GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS 4
#define GLOBAL_DISTANCE_FIELD_INVALID_PAGE_ID 0xFFFFFFFF

#define GLOBAL_DISTANCE_FIELD_PAGE_COVERAGE_BIT 0x80000000

// Coverage is a payload layer alongside the SDF
#define GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS 4 // Includes 0.5 texel trilinear filter margin
#define GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION (GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS - 1)
#define GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR (GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS / GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS)

struct FGlobalDistanceFieldPage
{
	uint PageIndex;
	bool bValid;
	bool bCoverage;
};

uint3 GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(FGlobalDistanceFieldPage Page)
{
	uint3 PageAtlasOffset;
	//PageAtlasOffset.x = (Page.PageIndex % GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_X);
	//PageAtlasOffset.y = ((Page.PageIndex / GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_X) % GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_Y);
	//PageAtlasOffset.z = ((Page.PageIndex / GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_X) / GLOBAL_DISTANCE_FIELD_PAGE_ATLAS_SIZE_IN_PAGES_Y);

	// Same as above, but with bit operations
	PageAtlasOffset.x = Page.PageIndex & 0x7F;
	PageAtlasOffset.y = (Page.PageIndex >> 7) & 0x7F;
	PageAtlasOffset.z = Page.PageIndex >> 14;

	return PageAtlasOffset;
}

uint PackGlobalDistanceFieldPage(FGlobalDistanceFieldPage Page)
{
	uint PackedPage = GLOBAL_DISTANCE_FIELD_INVALID_PAGE_ID;
	if (Page.bValid)
	{
		PackedPage = Page.PageIndex & 0x00FFFFFF;
		PackedPage |= Page.bCoverage ? GLOBAL_DISTANCE_FIELD_PAGE_COVERAGE_BIT : 0;
	}
	return PackedPage;
}

FGlobalDistanceFieldPage UnpackGlobalDistanceFieldPage(uint PackedPage)
{
	FGlobalDistanceFieldPage Page;
	Page.PageIndex = PackedPage & 0x00FFFFFF;
	Page.bCoverage = PackedPage & GLOBAL_DISTANCE_FIELD_PAGE_COVERAGE_BIT;
	Page.bValid = PackedPage < GLOBAL_DISTANCE_FIELD_INVALID_PAGE_ID;
	return Page;
}

FGlobalDistanceFieldPage GetGlobalDistanceFieldPage(float3 VolumeUV, uint ClipmapIndex)
{
	int4 PageTableCoord = int4(saturate(VolumeUV) * GlobalDistanceFieldClipmapSizeInPages + int3(0, 0, ClipmapIndex * GlobalDistanceFieldClipmapSizeInPages), 0);
	uint PackedPage = GlobalDistanceFieldPageTableTexture.Load(PageTableCoord);
	return UnpackGlobalDistanceFieldPage(PackedPage);
}

void ComputeGlobalDistanceFieldPageUV(float3 VolumeUV, FGlobalDistanceFieldPage Page, out float3 OutPageUV, out float3 OutCoveragePageUV)
{
	uint3 PageAtlasOffset = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page);
	float3 VolumePageUV = frac(VolumeUV * GlobalDistanceFieldClipmapSizeInPages);

	float3 PageAtlasCoord = PageAtlasOffset * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS + VolumePageUV * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + 0.5f;
	OutPageUV = PageAtlasCoord * GlobalDistanceFieldInvPageAtlasSize;

	float3 CoveragePageAtlasCoord = PageAtlasOffset * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS + VolumePageUV * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION + 0.5f;
	OutCoveragePageUV = CoveragePageAtlasCoord * GlobalDistanceFieldInvCoverageAtlasSize;
}

float3 ComputeGlobalDistanceFieldPageUV(float3 VolumeUV, FGlobalDistanceFieldPage Page)
{
	uint3 PageAtlasOffset = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page);
	float3 VolumePageUV = frac(VolumeUV * GlobalDistanceFieldClipmapSizeInPages);
	float3 PageAtlasCoord = PageAtlasOffset * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS + VolumePageUV * GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION + 0.5f;
	float3 PageUV = PageAtlasCoord * GlobalDistanceFieldInvPageAtlasSize;
	return PageUV;
}

float3 ComputeGlobalDistanceFieldCoveragePageUV(float3 VolumeUV, FGlobalDistanceFieldPage Page)
{
	uint3 PageAtlasOffset = GlobalDistanceFieldPageLinearIndexToPageAtlasOffset(Page);
	float3 VolumePageUV = frac(VolumeUV * GlobalDistanceFieldClipmapSizeInPages);
	float3 CoveragePageAtlasCoord = PageAtlasOffset * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION_IN_ATLAS + VolumePageUV * GLOBAL_DISTANCE_FIELD_COVERAGE_PAGE_RESOLUTION + 0.5f;
	float3 CoveragePageUV = CoveragePageAtlasCoord * GlobalDistanceFieldInvCoverageAtlasSize;
	return CoveragePageUV;
}

float EncodeGlobalDistanceFieldPageDistance(float Distance, float ClipmapInfluenceRange)
{
	return saturate(Distance / (2.0f * ClipmapInfluenceRange) + 0.5f);
}

float DecodeGlobalDistanceFieldPageDistance(float EncodedDistance, float ClipmapInfluenceRange)
{
	return (EncodedDistance * 2.0f - 1.0f) * ClipmapInfluenceRange;
}

float3 ComputeGlobalUV(float3 TranslatedWorldPosition, uint ClipmapIndex)
{
	//return ((TranslatedWorldPosition - GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz + GlobalVolumeScollOffset[ClipmapIndex].xyz) / (GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * 2) + .5f);
	float4 TranslatedWorldToUVAddAndMul = GlobalVolumeTranslatedWorldToUVAddAndMul[ClipmapIndex];
	float3 UV = frac(TranslatedWorldPosition * TranslatedWorldToUVAddAndMul.www + TranslatedWorldToUVAddAndMul.xyz); // wraparound addressing
	UV = frac(UV); // apply frac twice to prevent UV == 1.0f because frac(-0.00...001f) = 1.0f
	return UV;
}

float3 ComputeGlobalMipUV(float3 TranslatedWorldPosition, uint ClipmapIndex)
{
	float3 MipUV = saturate(TranslatedWorldPosition * GlobalDistanceFieldMipTranslatedWorldToUVScale[ClipmapIndex].xyz + GlobalDistanceFieldMipTranslatedWorldToUVBias[ClipmapIndex].xyz);

	// Clamp MipUV.z to a valid bilinear region to prevent filtering across clipmaps
	float MipUVMinZ = GlobalDistanceFieldMipTranslatedWorldToUVScale[ClipmapIndex].w;
	float MipUVMaxZ = GlobalDistanceFieldMipTranslatedWorldToUVBias[ClipmapIndex].w;
	MipUV.z = clamp(MipUV.z, MipUVMinZ, MipUVMaxZ);

	return MipUV;
}

float GetDistanceToNearestSurfaceGlobal(float3 TranslatedWorldPosition)
{
	float Distance = 0.0f;

	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ClipmapIndex++)
	{
		float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldPosition);
		const float ClipmapVoxelExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
		const float ClipmapInfluenceRange = 2.0f * GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS * ClipmapVoxelExtent;

		Distance = ClipmapInfluenceRange;

		if (DistanceFromClipmap > ClipmapVoxelExtent)
		{
			float3 ClipmapVolumeUV = ComputeGlobalUV(TranslatedWorldPosition, ClipmapIndex);

			FGlobalDistanceFieldPage Page = GetGlobalDistanceFieldPage(ClipmapVolumeUV, ClipmapIndex);
			if (Page.bValid)
			{
				float3 PageUV = ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page);

				float DistanceFieldValue = Texture3DSampleLevel(GlobalDistanceFieldPageAtlasTexture, GlobalDistanceFieldPageAtlasTextureSampler, PageUV, 0).x;

				if (DistanceFieldValue < 1.0f)
				{
					Distance = DecodeGlobalDistanceFieldPageDistance(DistanceFieldValue, ClipmapInfluenceRange);
					break;
				}
			}
		}
	}

	return Distance;
}

float SampleGlobalDistanceField(float3 TranslatedWorldPosition, float MaxDistance, int MinClipmapIndex)
{
	for (uint ClipmapIndex = MinClipmapIndex; ClipmapIndex < NumGlobalSDFClipmaps; ++ClipmapIndex)
	{
		const float ClipmapInfluenceRange = GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS * 2.0f * GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
		
		float3 ClipmapVolumeUV = ComputeGlobalUV(TranslatedWorldPosition, ClipmapIndex);
		FGlobalDistanceFieldPage Page = GetGlobalDistanceFieldPage(ClipmapVolumeUV, ClipmapIndex);

		if (Page.bValid)
		{
			float3 PageUV = ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page);

			float DistanceFieldValue = Texture3DSampleLevel(GlobalDistanceFieldPageAtlasTexture, GlobalDistanceFieldPageAtlasTextureSampler, PageUV, 0).x;

			if (DistanceFieldValue < 1.0f)
			{
				return DecodeGlobalDistanceFieldPageDistance(DistanceFieldValue, ClipmapInfluenceRange);
			}
		}
	}

	return MaxDistance;
}

float GlobalDistanceFieldSampleClipmap(float3 ClipmapVolumeUV, uint ClipmapIndex)
{
	float DistanceFieldValue = 1.0f;

	// Wrap UV sampling
	ClipmapVolumeUV = frac(ClipmapVolumeUV);

	FGlobalDistanceFieldPage Page = GetGlobalDistanceFieldPage(ClipmapVolumeUV, ClipmapIndex);
	if (Page.bValid)
	{
		float3 PageUV = ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page);
		DistanceFieldValue = Texture3DSampleLevel(GlobalDistanceFieldPageAtlasTexture, GlobalDistanceFieldPageAtlasTextureSampler, PageUV, 0).x;
	}

	return DistanceFieldValue;
}

float3 GlobalDistanceFieldPageCentralDiff(float3 ClipmapVolumeUV, uint ClipmapIndex)
{
	float3 TexelOffset = 0.5f * GlobalVolumeTexelSize;

	float R = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(+TexelOffset.x, 0, 0), ClipmapIndex);
	float L = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(-TexelOffset.x, 0, 0), ClipmapIndex);
	float F = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(0, +TexelOffset.y, 0), ClipmapIndex);
	float B = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(0, -TexelOffset.y, 0), ClipmapIndex);
	float U = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(0, 0, +TexelOffset.z), ClipmapIndex);
	float D = GlobalDistanceFieldSampleClipmap(ClipmapVolumeUV + float3(0, 0, -TexelOffset.z), ClipmapIndex);

	return float3(R - L, F - B, U - D);
}

float3 ComputeGlobalDistanceFieldNormal(float3 SampleTranslatedWorldPosition, uint ClipmapIndex, float3 FallbackNormal)
{
	float3 ClipmapVolumeUV = ComputeGlobalUV(SampleTranslatedWorldPosition, ClipmapIndex);
	float3 DistanceFieldGradient = GlobalDistanceFieldPageCentralDiff(ClipmapVolumeUV, ClipmapIndex);
	float DistanceFieldGradientLength = length(DistanceFieldGradient);
	float3 DistanceFieldNormal = DistanceFieldGradientLength > 0.001f ? DistanceFieldGradient / DistanceFieldGradientLength : FallbackNormal;
	return DistanceFieldNormal;
}

// Returns vector (unnormalized) to the nearest surface point.
// This vector is rescaled by 128 * GlobalVolumeTexelSize for backwards compatiblity.
float3 GetDistanceFieldGradientGlobal(float3 TranslatedWorldPosition)
{
	float3 Gradient = float3(0, 0, 0.001f);

	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ClipmapIndex++)
	{
		const float ClipmapVoxelExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
		const float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldPosition);

		if (DistanceFromClipmap >= ClipmapVoxelExtent)
		{
			const float3 ClipmapVolumeUV = ComputeGlobalUV(TranslatedWorldPosition, ClipmapIndex);
			FGlobalDistanceFieldPage Page = GetGlobalDistanceFieldPage(ClipmapVolumeUV, ClipmapIndex);

			if (Page.bValid)
			{
				float3 PageUV = ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page);

				float DistanceFieldValue = Texture3DSampleLevel(GlobalDistanceFieldPageAtlasTexture, GlobalDistanceFieldPageAtlasTextureSampler, PageUV, 0).x;

				// Find a page which where narrow band isn't clipped
				if (DistanceFieldValue < 0.9f)
				{
					float3 PageCentralDiff = GlobalDistanceFieldPageCentralDiff(ClipmapVolumeUV, ClipmapIndex);

					const float ClipmapInfluenceRange = GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS * 2.0f * ClipmapVoxelExtent;
					const float PageDistanceDecodeFactor = 2.0f * ClipmapInfluenceRange;
					float ClipmapExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w;

					// Rescale to keep backwards compatability
					const float BackwardsCompatibilityFactor = 128.0f * GlobalVolumeTexelSize;
					Gradient = PageDistanceDecodeFactor * PageCentralDiff / (ClipmapExtent * BackwardsCompatibilityFactor);

					break;
				}
			}
		}
	}

	return Gradient;
}

float CalculateDistanceFieldApproxAO(float3 TranslatedWorldPosition, float3 WorldNormal, uint NumSteps, float StepDistance, float StepScale, float DistanceBias, float MaxDistance)
{
	// determine the min clipmap that covers all the sampling points (sphere centered on TranslatedWorldPosition with radius = MaxDistance)
	int MinClipmapIndex = -1;

	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ++ClipmapIndex)
	{
		float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldPosition);
		if (DistanceFromClipmap > MaxDistance)
		{
			MinClipmapIndex = ClipmapIndex;
			break;
		}
	}

	if (MinClipmapIndex < 0)
	{
		return 1.0f;
	}

	float Occlusion = 0.0f;
	float W = 0.5f;
	float TotalW = 0.0f;

	for (uint StepIndex = 0; StepIndex < NumSteps; ++StepIndex)
	{
		const float CurrentDistance = StepDistance + DistanceBias;
		const float Dist = max(0.0f, SampleGlobalDistanceField(TranslatedWorldPosition + WorldNormal * CurrentDistance, CurrentDistance, MinClipmapIndex));

		Occlusion += W * Dist / CurrentDistance;
		TotalW += W;
		W *= 0.5f;

		StepDistance *= StepScale;
	}

	return saturate(Occlusion / TotalW);
}

#else

float GetDistanceToNearestSurfaceGlobal(float3 TranslatedWorldPosition)
{
	return MaxGlobalDFAOConeDistance;
}

float3 GetDistanceFieldGradientGlobal(float3 TranslatedWorldPosition)
{
	return float3(0, 0, .001f);
}

float CalculateDistanceFieldApproxAO(float3 TranslatedWorldPosition, float3 WorldNormal, uint NumSteps, float StepDistance, float StepScale, float DistanceBias, float MaxDistance)
{
	return 1.0f;
}

#endif

float GetDistanceToNearestSurfaceGlobal(FDFVector3 WorldPosition)
{
	const float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldPosition, ResolvedView.PreViewTranslation);
	return GetDistanceToNearestSurfaceGlobal(TranslatedWorldPosition);
}

float GetDistanceToNearestSurfaceGlobal(FLWCVector3 WorldPosition)
{
	return GetDistanceToNearestSurfaceGlobal(DFFromTileOffset(WorldPosition));
}

float3 GetDistanceFieldGradientGlobal(FDFVector3 WorldPosition)
{
	const float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldPosition, ResolvedView.PreViewTranslation);
	return GetDistanceFieldGradientGlobal(TranslatedWorldPosition);
}

float3 GetDistanceFieldGradientGlobal(FLWCVector3 WorldPosition)
{
	return GetDistanceFieldGradientGlobal(DFFromTileOffset(WorldPosition));
}

float CalculateDistanceFieldApproxAO(FDFVector3 WorldPosition, float3 WorldNormal, uint NumSteps, float StepDistance, float StepScale, float DistanceBias, float MaxDistance)
{
	const float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldPosition, ResolvedView.PreViewTranslation);
	return CalculateDistanceFieldApproxAO(TranslatedWorldPosition, WorldNormal, NumSteps, StepDistance, StepScale, DistanceBias, MaxDistance);
}

float CalculateDistanceFieldApproxAO(FLWCVector3 WorldPosition, float3 WorldNormal, uint NumSteps, float StepDistance, float StepScale, float DistanceBias, float MaxDistance)
{
	const float3 TranslatedWorldPosition = LWCToFloat(LWCAdd(WorldPosition, ResolvedView.TileOffset.PreViewTranslation));
	return CalculateDistanceFieldApproxAO(TranslatedWorldPosition, WorldNormal, NumSteps, StepDistance, StepScale, DistanceBias, MaxDistance);
}



