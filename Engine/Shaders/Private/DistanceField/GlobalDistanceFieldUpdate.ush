// Copyright Epic Games, Inc. All Rights Reserved.

#define CULL_GRID_FACTOR 4
#define GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X (GLOBAL_DISTANCE_FIELD_PAGE_RESOLUTION_IN_ATLAS / 4)
#define GLOBAL_DISTANCE_FIELD_PAGE_GROUP_NUM (GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X * GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X * GLOBAL_DISTANCE_FIELD_PAGE_GROUP_X)

uint3 UnpackPageUpdateTile(uint PackedPageTile)
{
	uint3 GridCoord;
	GridCoord.x = PackedPageTile & 0xFF;
	GridCoord.y = (PackedPageTile >> 8) & 0xFF;
	GridCoord.z = (PackedPageTile >> 16) & 0xFF;
	return GridCoord;
}

uint PackPageUpdateTile(uint3 GridCoord)
{
	return GridCoord.x | (GridCoord.y << 8) | (GridCoord.z << 16);
}

uint3 PageGridResolution;
uint PageTableClipmapOffsetZ;
float3 PageCoordToPageTranslatedWorldCenterScale;
float3 PageCoordToPageTranslatedWorldCenterBias;
float4 ClipmapVolumeTranslatedWorldToUVAddAndMul;

uint3 PageGridCoordToPageTableTextureCoord(uint3 PageGridCoord)
{
	float3 PageTranslatedWorldCenter = PageGridCoord * PageCoordToPageTranslatedWorldCenterScale + PageCoordToPageTranslatedWorldCenterBias;

	float3 ClipmapUV = frac(PageTranslatedWorldCenter * ClipmapVolumeTranslatedWorldToUVAddAndMul.www + ClipmapVolumeTranslatedWorldToUVAddAndMul.xyz);

	int3 PageTableTextureCoord = clamp(saturate(ClipmapUV) * PageGridResolution, 0, PageGridResolution - 1) + int3(0, 0, PageTableClipmapOffsetZ);
	return PageTableTextureCoord;
}

float ComputeSquaredDistanceBetweenAABBs(float3 CenterA, float3 ExtentA, float3 CenterB, float3 ExtentB)
{
	float3 AxisDistances = max(abs(CenterB - CenterA) - (ExtentA + ExtentB), 0);
	return dot(AxisDistances, AxisDistances);
}