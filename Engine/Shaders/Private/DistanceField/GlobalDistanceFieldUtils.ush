// Copyright Epic Games, Inc. All Rights Reserved.

// Controls how many voxels around a single sided object will be marked with full coverage
// This should be larger than or equal to r.LumenScene.GlobalSDF.NotCoveredMinStepScale to avoid artifacts (stepping too far)
#define ONE_SIDED_BAND_SIZE (4.0f * GLOBAL_DISTANCE_FIELD_COVERAGE_DOWNSAMPLE_FACTOR)

#ifndef GLOBALSDF_USE_COVERAGE_BASED_EXPAND
#define GLOBALSDF_USE_COVERAGE_BASED_EXPAND 1
#endif

#ifndef GLOBALSDF_SIMPLE_COVERAGE_BASED_EXPAND
#define GLOBALSDF_SIMPLE_COVERAGE_BASED_EXPAND 0
#endif

uint ComputeGlobalDistanceFieldClipmapIndex(float3 TranslatedWorldPosition)
{
	uint FoundClipmapIndex = 0;

	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ClipmapIndex++)
	{
		float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldPosition);

		if (DistanceFromClipmap > GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize)
		{
			FoundClipmapIndex = ClipmapIndex;
			break;
		}
	}

	return FoundClipmapIndex;
}

struct FGlobalSDFTraceInput
{
	float3 TranslatedWorldRayStart;
	float3 WorldRayDirection;
	float MinTraceDistance;
	float MaxTraceDistance;
	float StepFactor;
	float MinStepFactor;
	// The SDF surface is expanded to reduce leaking through thin surfaces, especially foliage meshes with bGenerateDistanceFieldAsIfTwoSided
	// Expanding as RayTime increases errors on the side of over-occlusion, especially at grazing angles, which can be desirable for diffuse GI.
	// Expanding as MaxDistance increases has less incorrect self-intersection which is desirable for reflections rays.
	bool bExpandSurfaceUsingRayTimeInsteadOfMaxDistance;
	float InitialMaxDistance;

	// Bias relative to the clipmap's voxel size
	float VoxelSizeRelativeBias;
	// Ray length bias relative to the clipmap's voxel size
	float VoxelSizeRelativeRayEndBias;

	// For dithered semi-transparency
	bool bDitheredTransparency;
	float2 DitherScreenCoord;
};

FGlobalSDFTraceInput SetupGlobalSDFTraceInput(float3 InTranslatedWorldRayStart, float3 InWorldRayDirection, float InMinTraceDistance, float InMaxTraceDistance, float InStepFactor, float InMinStepFactor)
{
	FGlobalSDFTraceInput TraceInput;
	TraceInput.TranslatedWorldRayStart = InTranslatedWorldRayStart;
	TraceInput.WorldRayDirection = InWorldRayDirection;
	TraceInput.MinTraceDistance = InMinTraceDistance;
	TraceInput.MaxTraceDistance = InMaxTraceDistance;
	TraceInput.StepFactor = InStepFactor;
	TraceInput.MinStepFactor = InMinStepFactor;
	TraceInput.bExpandSurfaceUsingRayTimeInsteadOfMaxDistance = true;
	TraceInput.InitialMaxDistance = 0;
	TraceInput.VoxelSizeRelativeBias = 0.0f;
	TraceInput.VoxelSizeRelativeRayEndBias = 0.0f;
	TraceInput.bDitheredTransparency = false;
	TraceInput.DitherScreenCoord = float2(0, 0);
	return TraceInput;
}

struct FGlobalSDFTraceResult
{
	float HitTime;
	uint HitClipmapIndex;
	uint TotalStepsTaken;

	// Amount the surface was expanded at the hit
	float ExpandSurfaceAmount;
};

bool GlobalSDFTraceResultIsHit(FGlobalSDFTraceResult TraceResult)
{ 
	return TraceResult.HitTime >= 0.0f;
}

FGlobalSDFTraceResult RayTraceGlobalDistanceField(FGlobalSDFTraceInput TraceInput)
{
	FGlobalSDFTraceResult TraceResult;
	TraceResult.HitTime = -1.0f;
	TraceResult.HitClipmapIndex = 0;
	TraceResult.TotalStepsTaken = 0;
	TraceResult.ExpandSurfaceAmount = 0;

	float TraceNoise = InterleavedGradientNoise(TraceInput.DitherScreenCoord.xy, View.StateFrameIndexMod8);
	uint MinClipmapIndex = ComputeGlobalDistanceFieldClipmapIndex(TraceInput.TranslatedWorldRayStart + TraceInput.MinTraceDistance * TraceInput.WorldRayDirection);
	float MaxDistance = TraceInput.InitialMaxDistance;
	float MinRayTime = TraceInput.MinTraceDistance;

	LOOP
	for (uint ClipmapIndex = MinClipmapIndex; ClipmapIndex < NumGlobalSDFClipmaps && TraceResult.HitTime < 0.0f; ++ClipmapIndex)
	{
		float ClipmapVoxelExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
		float MinStepSize = TraceInput.MinStepFactor * ClipmapVoxelExtent;
		float ExpandSurfaceDistance = ClipmapVoxelExtent;
		float ClipmapRayBias = ClipmapVoxelExtent * TraceInput.VoxelSizeRelativeBias;
		float ClipmapRayLength = TraceInput.MaxTraceDistance - ClipmapVoxelExtent * TraceInput.VoxelSizeRelativeRayEndBias;

		float3 GlobalVolumeTranslatedCenter = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz;
		float GlobalVolumeExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w - ClipmapVoxelExtent;
		float3 TranslatedWorldRayEnd = TraceInput.TranslatedWorldRayStart + TraceInput.WorldRayDirection * ClipmapRayLength;
		float2 IntersectionTimes = LineBoxIntersect(TraceInput.TranslatedWorldRayStart, TranslatedWorldRayEnd, GlobalVolumeTranslatedCenter - GlobalVolumeExtent.xxx, GlobalVolumeTranslatedCenter + GlobalVolumeExtent.xxx);
		IntersectionTimes.xy *= ClipmapRayLength;
		IntersectionTimes.x = max(IntersectionTimes.x, MinRayTime);
		IntersectionTimes.x = max(IntersectionTimes.x, ClipmapRayBias);

		if (IntersectionTimes.x < IntersectionTimes.y)
		{
			// Update the trace start for the next clipmap
			MinRayTime = IntersectionTimes.y;

			float SampleRayTime = IntersectionTimes.x;
			const float ClipmapInfluenceRange = GLOBAL_DISTANCE_FIELD_INFLUENCE_RANGE_IN_VOXELS * 2.0f * GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;

			uint StepIndex = 0;
			const uint MaxSteps = 256;

			LOOP
			for (; StepIndex < MaxSteps; ++StepIndex)
			{
				float3 SampleTranslatedWorldPosition = TraceInput.TranslatedWorldRayStart + TraceInput.WorldRayDirection * SampleRayTime;

				float3 ClipmapVolumeUV = ComputeGlobalUV(SampleTranslatedWorldPosition, ClipmapIndex);
				float3 MipUV = ComputeGlobalMipUV(SampleTranslatedWorldPosition, ClipmapIndex);

				float DistanceFieldMipValue = Texture3DSampleLevel(GlobalDistanceFieldMipTexture, GlobalDistanceFieldMipTextureSampler, MipUV, 0).x;
				float DistanceField = DecodeGlobalDistanceFieldPageDistance(DistanceFieldMipValue, GlobalDistanceFieldMipFactor * ClipmapInfluenceRange);
				float Coverage = 1;
				FGlobalDistanceFieldPage Page = GetGlobalDistanceFieldPage(ClipmapVolumeUV, ClipmapIndex);

				if (Page.bValid && DistanceFieldMipValue < GlobalDistanceFieldMipTransition)
				{
					float3 PageUV = ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page);

					#if GLOBALSDF_USE_COVERAGE_BASED_EXPAND
					if (Page.bCoverage)
					{
					#if GLOBALSDF_SIMPLE_COVERAGE_BASED_EXPAND
						Coverage = 0;
					#else
						float3 CoveragePageUV;
						ComputeGlobalDistanceFieldPageUV(ClipmapVolumeUV, Page, PageUV, CoveragePageUV);
						Coverage = Texture3DSampleLevel(GlobalDistanceFieldCoverageAtlasTexture, GlobalDistanceFieldCoverageAtlasTextureSampler, CoveragePageUV, 0).x;
					#endif
					}
					#endif

					float DistanceFieldValue = Texture3DSampleLevel(GlobalDistanceFieldPageAtlasTexture, GlobalDistanceFieldPageAtlasTextureSampler, PageUV, 0).x;
					DistanceField = DecodeGlobalDistanceFieldPageDistance(DistanceFieldValue, ClipmapInfluenceRange);
				}

				MaxDistance = max(DistanceField, MaxDistance);

				float ExpandSurfaceTime = TraceInput.bExpandSurfaceUsingRayTimeInsteadOfMaxDistance ? SampleRayTime - ClipmapRayBias : MaxDistance;
				float ExpandSurfaceScale = lerp(NotCoveredExpandSurfaceScale, CoveredExpandSurfaceScale, Coverage);

				const float ExpandSurfaceFalloff = 2.0f * ExpandSurfaceDistance;
				const float ExpandSurfaceAmount = ExpandSurfaceDistance * saturate(ExpandSurfaceTime / ExpandSurfaceFalloff) * ExpandSurfaceScale;

				float StepNoise = InterleavedGradientNoise(TraceInput.DitherScreenCoord.xy, View.StateFrameIndexMod8 * MaxSteps + StepIndex);

				if (DistanceField < ExpandSurfaceAmount 
					&& (!TraceInput.bDitheredTransparency || (StepNoise * (1 - Coverage) <= DitheredTransparencyStepThreshold && TraceNoise * (1 - Coverage) <= DitheredTransparencyTraceThreshold)))
				{
					TraceResult.HitTime = max(SampleRayTime + DistanceField - ExpandSurfaceAmount, 0.0f);
					TraceResult.HitClipmapIndex = ClipmapIndex;
					TraceResult.ExpandSurfaceAmount = ExpandSurfaceAmount;
					break;
				}

				float LocalMinStepSize = MinStepSize * lerp(NotCoveredMinStepScale, 1.0f, Coverage);
				float StepDistance = max(DistanceField * TraceInput.StepFactor, LocalMinStepSize);
				SampleRayTime += StepDistance;

				// Terminate the trace if we went past the end of the ray or achieved enough occlusion
				if (SampleRayTime > IntersectionTimes.y || TraceResult.HitTime >= 0.0f)
				{
					break;
				}
			}

			TraceResult.TotalStepsTaken += StepIndex;
		}
	}

	return TraceResult;
}

void ClipRayToStartOutsideGlobalSDF(inout float3 TranslatedRayOrigin, float3 RayDirection, inout float RayLength)
{
	uint ClipmapIndex = NumGlobalSDFClipmaps - 1;

	float3 TranslatedWorldRayEnd = TranslatedRayOrigin + RayDirection * RayLength;
	float ClipmapVoxelExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
	float GlobalVolumeExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w - ClipmapVoxelExtent;
	float3 GlobalVolumeTranslatedCenter = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz;
	float2 IntersectionTimes = LineBoxIntersect(TranslatedRayOrigin, TranslatedWorldRayEnd, GlobalVolumeTranslatedCenter - GlobalVolumeExtent.xxx, GlobalVolumeTranslatedCenter + GlobalVolumeExtent.xxx);

	if (IntersectionTimes.x < IntersectionTimes.y)
	{
		// Pull the origin inside the box slightly to reduce the gap created by jittering ray step offset
		// t + (t - 1) / (n * 2 - 1) where n is the number of distant screen trace steps
		IntersectionTimes.y = max((32.f / 31.f) * IntersectionTimes.y - 1.f / 31.f, 0);

		TranslatedRayOrigin += RayDirection * IntersectionTimes.y * RayLength;
		RayLength = max(RayLength * (1.0f - IntersectionTimes.y), 0);
	}
}