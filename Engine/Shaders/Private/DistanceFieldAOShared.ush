// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldAOShared.ush
=============================================================================*/

#include "IntersectionUtils.ush"

// Must match C++
#define NUM_CONE_STEPS 10

// Must match C++
#define NUM_CONE_DIRECTIONS 9

// Must match C++
#define AO_DOWNSAMPLE_FACTOR 2

float AOObjectMaxDistance;
float AOStepScale;
float AOStepExponentScale;
float AOMaxViewDistance;
float DistanceFadeScale; // 1 / ((1 - FadeDistanceFraction) * AOMaxViewDistance)

float AOGlobalMaxOcclusionDistance;

// Gives the max occlusion distance regardless of whether the global distance field is in use
float GetAOMaxDistance()
{
	return max(AOObjectMaxDistance, AOGlobalMaxOcclusionDistance);
}

float GetStepOffset(float StepIndex)
{
	// Original heuristic
	//return AOStepScale * exp2(AOStepExponentScale * StepIndex);

	float temp = AOStepExponentScale * StepIndex;
	return AOStepScale * (temp * temp + 1);
}

uint2 TileListGroupSize;
 
Texture2D DistanceFieldNormalTexture;
SamplerState DistanceFieldNormalSampler;

float4 EncodeDownsampledGBuffer(in float3 WorldNormal, float SceneDepth)
{
	#if METAL_ES3_1_PROFILE 
		// clamp max depth to avoid #inf
		SceneDepth = min(SceneDepth, 65500.0f);
	#endif
	return float4(WorldNormal, SceneDepth);
} 

void GetDownsampledGBuffer(float2 ScreenUV, out float3 OutNormal, out float OutDepth)
{
	float4 TextureValue = Texture2DSampleLevel(DistanceFieldNormalTexture, DistanceFieldNormalSampler, ScreenUV, 0);
	OutNormal = TextureValue.xyz;
	OutDepth = TextureValue.w;
}

float GetDownsampledDepth(float2 ScreenUV)
{
	return abs(Texture2DSampleLevel(DistanceFieldNormalTexture, DistanceFieldNormalSampler, ScreenUV, 0).w);
}

float2 BaseLevelTexelSize;

Texture2D BentNormalAOTexture;
SamplerState BentNormalAOSampler;

Buffer<float> RecordConeVisibility;
float4 SampleDirections[NUM_CONE_DIRECTIONS];
float BentNormalNormalizeFactor;

void FindBestAxisVectors2(float3 InZAxis, out float3 OutXAxis, out float3 OutYAxis )
{
	float3 UpVector = abs(InZAxis.z) < 0.999 ? float3(0,0,1) : float3(1,0,0);
	OutXAxis = normalize( cross( UpVector, InZAxis ) );
	OutYAxis = cross( InZAxis, OutXAxis );
}

float3 ComputeBentNormal(float3 RecordWorldNormal, uint RelativeRecordIndex)
{
	float3 TangentX;
	float3 TangentY;
	FindBestAxisVectors2(RecordWorldNormal, TangentX, TangentY);

	float3 UnoccludedDirection = 0;

	for (uint ConeIndex = 0; ConeIndex < NUM_CONE_DIRECTIONS; ConeIndex++)
	{
		float3 ConeDirection = SampleDirections[ConeIndex].xyz;
		float3 RotatedConeDirection = ConeDirection.x * TangentX + ConeDirection.y * TangentY + ConeDirection.z * RecordWorldNormal;

		float ConeVisibility = RecordConeVisibility[RelativeRecordIndex * NUM_CONE_DIRECTIONS + ConeIndex];
		UnoccludedDirection += ConeVisibility * RotatedConeDirection;
	}

	float InvNumSamples = 1.0f / (float)NUM_CONE_DIRECTIONS;
	UnoccludedDirection = UnoccludedDirection * (BentNormalNormalizeFactor * InvNumSamples);

	return UnoccludedDirection;
}

float2 AOBufferBilinearUVMax;

struct FUpsampleDFAOOutput
{
	float3 BentNormal;
	float FadeAlpha;
};
FUpsampleDFAOOutput UpsampleDFAO(float2 BufferUV, float SceneDepth)
{
	FUpsampleDFAOOutput UpsampleDFAOOutput;

	float2 DistanceFieldUVs = min(BufferUV, AOBufferBilinearUVMax);

#define BILATERAL_UPSAMPLE 1
#if BILATERAL_UPSAMPLE
	float2 LowResBufferSize = floor(View.BufferSizeAndInvSize.xy / AO_DOWNSAMPLE_FACTOR);
	float2 LowResTexelSize = 1.0f / LowResBufferSize;
	float2 Corner00UV = floor(DistanceFieldUVs * LowResBufferSize - .5f) / LowResBufferSize + .5f * LowResTexelSize;
	float2 BilinearWeights = (DistanceFieldUVs - Corner00UV) * LowResBufferSize;

	float4 TextureValues00 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, Corner00UV, 0);
	float4 TextureValues10 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, Corner00UV + float2(LowResTexelSize.x, 0), 0);
	float4 TextureValues01 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, Corner00UV + float2(0, LowResTexelSize.y), 0);
	float4 TextureValues11 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, Corner00UV + LowResTexelSize, 0);

	float4 CornerWeights = float4(
		(1 - BilinearWeights.y) * (1 - BilinearWeights.x),
		(1 - BilinearWeights.y) * BilinearWeights.x,
		BilinearWeights.y * (1 - BilinearWeights.x),
		BilinearWeights.y * BilinearWeights.x);

	float Epsilon = .0001f;

	float4 CornerDepths = float4(TextureValues00.w, TextureValues10.w, TextureValues01.w, TextureValues11.w);
	float4 DepthWeights = 1.0f / (abs(CornerDepths - SceneDepth.xxxx) + Epsilon);
	float4 FinalWeights = CornerWeights * DepthWeights;

	float InvWeight = 1.0f / dot(FinalWeights, 1);

	float3 InterpolatedResult =
		(FinalWeights.x * TextureValues00.xyz
			+ FinalWeights.y * TextureValues10.xyz
			+ FinalWeights.z * TextureValues01.xyz
			+ FinalWeights.w * TextureValues11.xyz)
		* InvWeight;

	float3 BentNormal = InterpolatedResult.xyz;

#else
	float3 BentNormal = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, DistanceFieldUVs, 0).xyz;

#endif

	UpsampleDFAOOutput.BentNormal = BentNormal;
	// Fade to unoccluded in the distance
	UpsampleDFAOOutput.FadeAlpha = saturate((AOMaxViewDistance - SceneDepth) * DistanceFadeScale);

	return UpsampleDFAOOutput;
}

float3 ApplyDFAO(FUpsampleDFAOOutput UpsampleDFAOOutput, float3 WorldNormal)
{
	return lerp(WorldNormal, UpsampleDFAOOutput.BentNormal, UpsampleDFAOOutput.FadeAlpha);
}

float3 UpsampleDFAO(float2 BufferUV, float SceneDepth, float3 WorldNormal)
{
	FUpsampleDFAOOutput UpsampleDFAOOutput = UpsampleDFAO(BufferUV, SceneDepth);
	return ApplyDFAO(UpsampleDFAOOutput, WorldNormal);
}

#ifndef CULLED_TILE_SIZEX
#define CULLED_TILE_SIZEX 4
#endif
 
#ifndef TRACE_DOWNSAMPLE_FACTOR
#define TRACE_DOWNSAMPLE_FACTOR 1
#endif
 
#ifndef CONE_TRACE_OBJECTS_THREADGROUP_SIZE
#define CONE_TRACE_OBJECTS_THREADGROUP_SIZE 16
#endif

// Size of a culled tile at the resolution that cone tracing is done, in one dimension
#define CONE_TILE_SIZEX (CULLED_TILE_SIZEX / TRACE_DOWNSAMPLE_FACTOR) 
// Number of culled tiles per cone tracing threadgroup
#define CONE_TRACE_TILES_PER_THREADGROUP (CONE_TRACE_OBJECTS_THREADGROUP_SIZE / (CONE_TILE_SIZEX * CONE_TILE_SIZEX))

uint2 ScreenGridConeVisibilitySize;
float2 JitterOffset;

uint2 ComputeTileCoordinateFromScreenGrid(uint2 OutputCoordinate)
{
	uint2 TileCoordinate = OutputCoordinate * TRACE_DOWNSAMPLE_FACTOR / CULLED_TILE_SIZEX;
	return TileCoordinate;
}

float2 GetBaseLevelScreenUVFromScreenGrid(uint2 OutputCoordinate, float JitterScale)
{
	float2 BaseLevelScreenUV = (OutputCoordinate * TRACE_DOWNSAMPLE_FACTOR + JitterOffset * JitterScale + float2(.5f, .5f)) * BaseLevelTexelSize;
	return BaseLevelScreenUV;
}

float2 GetBaseLevelScreenUVFromScreenGrid(uint2 OutputCoordinate)
{
	return GetBaseLevelScreenUVFromScreenGrid(OutputCoordinate, 1);
}

float2 GetScreenUVFromScreenGrid(uint2 OutputCoordinate, float JitterScale)
{
	float2 ScreenUV = ((OutputCoordinate * TRACE_DOWNSAMPLE_FACTOR + JitterOffset * JitterScale) * AO_DOWNSAMPLE_FACTOR + View.ViewRectMin.xy + float2(.5f, .5f)) * View.BufferSizeAndInvSize.zw;
	return ScreenUV;
}

float2 GetScreenUVFromScreenGrid(uint2 OutputCoordinate)
{
	return GetScreenUVFromScreenGrid(OutputCoordinate, 1);
}

StructuredBuffer<uint> CulledTilesStartOffsetArray;

#define INVALID_TILE_INDEX 0xFFFF

#ifndef CULLED_TILE_DATA_STRIDE
#define CULLED_TILE_DATA_STRIDE 1
#endif
