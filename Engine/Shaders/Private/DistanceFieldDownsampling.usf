// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldDownsampling.usf
=============================================================================*/

#include "Common.ush"


float3 TexelSrcSize;
uint4 DstSize;
uint4 OffsetInAtlas;
SamplerState MeshDFSampler;
Texture3D<float> MeshDF;
RWTexture3D<float> DFAtlas;

[numthreads(8,8,1)]
void DistanceFieldDownsamplingCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (all(ThreadId.xyz < DstSize.xyz))  
	{
		float3 VolumeUV = float3(ThreadId.xyz) / float3(DstSize.xyz);
		VolumeUV += TexelSrcSize;
		int3 TexturePixelOffset = ThreadId.xyz + (int3)OffsetInAtlas.xyz;
		DFAtlas[TexturePixelOffset] = MeshDF.SampleLevel(MeshDFSampler, VolumeUV, 0).x;
	}
}
