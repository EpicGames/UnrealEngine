// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldLightingPost.usf
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "DistanceFieldLightingShared.ush"
#include "DistanceFieldAOShared.ush"
#include "ScreenPass.ush"

Texture2D BentNormalHistoryTexture;
SamplerState BentNormalHistorySampler;
Texture2D IrradianceHistoryTexture;
SamplerState IrradianceHistorySampler;
float HistoryWeight;
float HistoryDistanceThreshold;
float UseHistoryFilter;

float4 HistoryScreenPositionScaleBias;
float4 HistoryUVMinMax;

Texture2D VelocityTexture;
SamplerState VelocityTextureSampler;

float ComputeHistoryWeightBasedOnPosition(float2 ScreenPosition, float SceneDepth, float2 OldScreenPosition, float HistoryCameraDepth)
{
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
	float3 PrevPositionTranslatedWorld = mul(float4(GetScreenPositionForProjectionType(OldScreenPosition, HistoryCameraDepth), HistoryCameraDepth, 1), View.PrevScreenToTranslatedWorld).xyz;

	float DistanceToHistoryValue = length(PrevPositionTranslatedWorld - TranslatedWorldPosition);
	float RelativeHistoryDistanceThreshold = HistoryDistanceThreshold / 1000.0f;

	return DistanceToHistoryValue / SceneDepth > RelativeHistoryDistanceThreshold ? 0.0f : 1.0f;
}

float2 DistanceFieldGBufferTexelSize;
float2 DistanceFieldGBufferJitterOffset;
float4 BentNormalBufferAndTexelSize;

FScreenTransform UVToScreenPos;

float4 GetNormalWeights(float2 Corner00UV, float2 LowResTexelSize, float3 WorldNormal)
{
	float4 NormalWeights;

	{
		float3 SampleWorldNormal;
		float Unused;
		GetDownsampledGBuffer(Corner00UV, SampleWorldNormal, Unused);
		NormalWeights.x = dot(SampleWorldNormal, WorldNormal);
	}

	{
		float3 SampleWorldNormal;
		float Unused;
		GetDownsampledGBuffer(Corner00UV + float2(LowResTexelSize.x, 0), SampleWorldNormal, Unused);
		NormalWeights.y = dot(SampleWorldNormal, WorldNormal);
	}

	{
		float3 SampleWorldNormal;
		float Unused;
		GetDownsampledGBuffer(Corner00UV + float2(0, LowResTexelSize.y), SampleWorldNormal, Unused);
		NormalWeights.z = dot(SampleWorldNormal, WorldNormal);
	}

	{
		float3 SampleWorldNormal;
		float Unused;
		GetDownsampledGBuffer(Corner00UV + LowResTexelSize, SampleWorldNormal, Unused);
		NormalWeights.w = dot(SampleWorldNormal, WorldNormal);
	}

	return max(NormalWeights, .0001f);
}

float ComputeSampleWeightBasedOnPosition(float4 ReferencePlane, float2 SampleScreenUV, float SampleDepth)
{
	float2 SampleScreenPosition = ApplyScreenTransform(SampleScreenUV, UVToScreenPos);
	float3 SampleTranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(SampleScreenPosition, SampleDepth), SampleDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	float PlaneDistance = dot(ReferencePlane, float4(SampleTranslatedWorldPosition, 1));

	float Epsilon = .0001f;
	float RelativeDistance = 1000 * abs(PlaneDistance) / SampleDepth;
	return min(10.0f / (RelativeDistance + Epsilon), 1);
}

void GeometryAwareUpsample(float4 UVAndScreenPos, out float4 OutBentNormal)
{
	float3 WorldNormal;
	float SceneDepth;
	GetDownsampledGBuffer(UVAndScreenPos.xy, WorldNormal, SceneDepth);

	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(UVAndScreenPos.zw, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	float4 ReferencePlane = float4(WorldNormal, -dot(TranslatedWorldPosition, WorldNormal));

	float2 LowResBufferSize = BentNormalBufferAndTexelSize.xy;
	float2 LowResTexelSize = BentNormalBufferAndTexelSize.zw;
	float2 Corner00UV = floor((UVAndScreenPos.xy - DistanceFieldGBufferJitterOffset) * LowResBufferSize) * LowResTexelSize;
	float2 BilinearWeights = (UVAndScreenPos.xy - Corner00UV - DistanceFieldGBufferJitterOffset) * LowResBufferSize;
	float2 LowResCorner00UV = Corner00UV + .5f * LowResTexelSize;

	float4 TextureValues00 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, LowResCorner00UV, 0);
	float4 TextureValues10 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, LowResCorner00UV + float2(LowResTexelSize.x, 0), 0);
	float4 TextureValues01 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, LowResCorner00UV + float2(0, LowResTexelSize.y), 0);
	float4 TextureValues11 = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, LowResCorner00UV + LowResTexelSize, 0);

	float4 CornerWeights = float4(
		(1 - BilinearWeights.y) * (1 - BilinearWeights.x),
		(1 - BilinearWeights.y) * BilinearWeights.x,
		BilinearWeights.y * (1 - BilinearWeights.x),
		BilinearWeights.y * BilinearWeights.x);

	float4 CornerDepths = float4(TextureValues00.w, TextureValues10.w, TextureValues01.w, TextureValues11.w);

	float4 PositionWeights;
	PositionWeights.x = ComputeSampleWeightBasedOnPosition(ReferencePlane, LowResCorner00UV, CornerDepths.x);
	PositionWeights.y = ComputeSampleWeightBasedOnPosition(ReferencePlane, LowResCorner00UV + float2(LowResTexelSize.x, 0), CornerDepths.y);
	PositionWeights.z = ComputeSampleWeightBasedOnPosition(ReferencePlane, LowResCorner00UV + float2(0, LowResTexelSize.y), CornerDepths.z);
	PositionWeights.w = ComputeSampleWeightBasedOnPosition(ReferencePlane, LowResCorner00UV + LowResTexelSize, CornerDepths.w);

	//float4 DepthWeights = max(exp2(-abs(CornerDepths - SceneDepth.xxxx) * .01f), .001f);
	float Epsilon = .0001f;
	//float4 DepthWeights = min(10.0f / (abs(CornerDepths - SceneDepth.xxxx) + Epsilon), 1);

	float2 FullResCorner00UV = Corner00UV + DistanceFieldGBufferJitterOffset + 0.5f * DistanceFieldGBufferTexelSize;
	float4 NormalWeights = GetNormalWeights(FullResCorner00UV, LowResTexelSize, WorldNormal);

	float4 FinalWeights = CornerWeights * PositionWeights * NormalWeights;

	float InvSafeWeight = 1.0f / max(dot(FinalWeights, 1), .00001f);

	float3 AverageBentNormal =
		(FinalWeights.x * TextureValues00.xyz
			+ FinalWeights.y * TextureValues10.xyz
			+ FinalWeights.z * TextureValues01.xyz
			+ FinalWeights.w * TextureValues11.xyz)
		* InvSafeWeight;

	OutBentNormal = float4(AverageBentNormal, SceneDepth);

	float BentNormalLength = length(OutBentNormal.rgb);
	float3 NormalizedBentNormal = OutBentNormal.rgb / max(BentNormalLength, .0001f);
	OutBentNormal.rgb = NormalizedBentNormal * BentNormalLength;
	//OutBentNormal = float4(WorldNormal, SceneDepth);
}

void GeometryAwareUpsamplePS(
	in float4 UVAndScreenPos : TEXCOORD0
	, out float4 OutBentNormal : SV_Target0
	)
{
	GeometryAwareUpsample(UVAndScreenPos, OutBentNormal);
}

/** Reproject the occlusion history. */
void UpdateHistoryDepthRejectionPS(
	in float4 UVAndScreenPos : TEXCOORD0
	,out float4 OutBentNormal : SV_Target0
	)
{
	float4 NewValue;
	GeometryAwareUpsample(UVAndScreenPos, NewValue);

	float SceneDepth = NewValue.w;

	float4 ThisClip = float4( UVAndScreenPos.zw, ConvertToDeviceZ(SceneDepth), 1 );
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float2 PrevScreen = PrevClip.xy / PrevClip.w;
	float2 ScreenVelocity = UVAndScreenPos.zw - PrevScreen;
		 
	// The scene velocity texture is rendered as a texture atlas for all views,
	// so first convert the vertex shader's UV coordinates to the size of the view rect and offset into the atlas
	// before converting back to the correct UV coordinates.
	float2 FullResTexel = (UVAndScreenPos.xy * View.ViewSizeAndInvSize.xy + View.ViewRectMin.xy - .5f) * View.BufferSizeAndInvSize.zw;
	float4 EncodedVelocity = Texture2DSampleLevel(VelocityTexture, VelocityTextureSampler, FullResTexel, 0);
		
	if (EncodedVelocity.x > 0.0)
	{
		ScreenVelocity = DecodeVelocityFromTexture(EncodedVelocity).xy;
	}
		
	float PixelSpeed = 0.5 * length(ScreenVelocity);

	float2 OldScreenPosition = (UVAndScreenPos.zw - ScreenVelocity);
	float2 OldDistanceFieldUVs = OldScreenPosition * HistoryScreenPositionScaleBias.xy + HistoryScreenPositionScaleBias.wz;

	float EffectiveHistoryWeight = HistoryWeight;

	FLATTEN
	if (any(OldDistanceFieldUVs > HistoryUVMinMax.zw) || any(OldDistanceFieldUVs < HistoryUVMinMax.xy))
	{
		EffectiveHistoryWeight = 0;
	}

	// It's not enough just to set EffectiveHistoryWeight to 0, as sampling from invalid history region may result in a NaN
	OldDistanceFieldUVs = clamp(OldDistanceFieldUVs, HistoryUVMinMax.xy, HistoryUVMinMax.zw);

// Manual resample disabled as it doesn't affect the artifacts that cause most ghosting
#define MANUAL_HISTORY_RESAMPLE 0
#if MANUAL_HISTORY_RESAMPLE

		float2 HistoryBufferSize = floor(View.BufferSizeAndInvSize.xy / DOWNSAMPLE_FACTOR);
		float2 HistoryTexelSize = View.BufferSizeAndInvSize.zw * DOWNSAMPLE_FACTOR;
		float2 OldDistanceFieldUVsCorner00 = HistoryTexelSize * (floor(OldDistanceFieldUVs * HistoryBufferSize - .5f) + .5f);
		float2 OldDistanceFieldUVsCorner01 = OldDistanceFieldUVsCorner00 + float2(0, HistoryTexelSize.y);
		float2 OldDistanceFieldUVsCorner11 = OldDistanceFieldUVsCorner00 + HistoryTexelSize;
		float2 OldDistanceFieldUVsCorner10 = OldDistanceFieldUVsCorner00 + float2(HistoryTexelSize.x, 0);
		float2 LerpWeights = (OldDistanceFieldUVs - OldDistanceFieldUVsCorner00) * HistoryBufferSize;

		float4 Corner00Value = Texture2DSampleLevel(BentNormalHistoryTexture, BentNormalHistorySampler, OldDistanceFieldUVsCorner00, 0);
		float4 Corner01Value = Texture2DSampleLevel(BentNormalHistoryTexture, BentNormalHistorySampler, OldDistanceFieldUVsCorner01, 0);
		float4 Corner11Value = Texture2DSampleLevel(BentNormalHistoryTexture, BentNormalHistorySampler, OldDistanceFieldUVsCorner11, 0);
		float4 Corner10Value = Texture2DSampleLevel(BentNormalHistoryTexture, BentNormalHistorySampler, OldDistanceFieldUVsCorner10, 0);

	#define COMPUTE_SEPARATE_POSITION_WEIGHTS 1
	#if COMPUTE_SEPARATE_POSITION_WEIGHTS
		float PositionWeight00 = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, Corner00Value.w);
		float PositionWeight01 = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, Corner01Value.w);
		float PositionWeight11 = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, Corner11Value.w);
		float PositionWeight10 = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, Corner10Value.w);

		float BilinearWeight00 = (1 - LerpWeights.y) * (1 - LerpWeights.x);
		float BilinearWeight01 = (LerpWeights.y) * (1 - LerpWeights.x);
		float BilinearWeight11 = (LerpWeights.y) * (LerpWeights.x);
		float BilinearWeight10 = (1 - LerpWeights.y) * (LerpWeights.x);

		float3 HistoryValue = (BilinearWeight00 * lerp(NewValue.xyz, Corner00Value.xyz, PositionWeight00)
			+ BilinearWeight01 * lerp(NewValue.xyz, Corner01Value.xyz, PositionWeight01)
			+ BilinearWeight11 * lerp(NewValue.xyz, Corner11Value.xyz, PositionWeight11)
			+ BilinearWeight10 * lerp(NewValue.xyz, Corner10Value.xyz, PositionWeight10));

		OutBentNormal.rgb = lerp(NewValue.rgb, HistoryValue.rgb, EffectiveHistoryWeight);

		EffectiveHistoryWeight *= (PositionWeight00 > 0 || PositionWeight01 > 0 || PositionWeight11 > 0 || PositionWeight10 > 0) ? 1.0f : 0.0f;
		
#define BENT_NORMAL_LENGTH_FIXUP 0
#if BENT_NORMAL_LENGTH_FIXUP

		float BentNormalLength = length(HistoryValue.xyz);

		float LengthVerticalLerp0 = lerp(length(Corner00Value.xyz), length(Corner01Value.xyz), LerpWeights.y);
		float LengthVerticalLerp1 = lerp(length(Corner10Value.xyz), length(Corner11Value.xyz), LerpWeights.y);
		float AverageLength = lerp(LengthVerticalLerp0, LengthVerticalLerp1, LerpWeights.x);

		if (BentNormalLength < AverageLength && BentNormalLength > 0)
		{
			// Fixup normal shortening due to weighted average of vectors
			HistoryValue.xyz = HistoryValue.xyz / BentNormalLength * AverageLength;
		}

#endif

	#else
		// Manually implement bilinear filtering with a single position weight, for testing
		float4 VerticalLerp0 = lerp(Corner00Value, Corner01Value, LerpWeights.y);
		float4 VerticalLerp1 = lerp(Corner10Value, Corner11Value, LerpWeights.y);
		float4 HistoryValue = lerp(VerticalLerp0, VerticalLerp1, LerpWeights.x);

		float PositionWeight = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, HistoryValue.w);
		EffectiveHistoryWeight *= PositionWeight;

		OutBentNormal.rgb = lerp(NewValue.rgb, HistoryValue.rgb, EffectiveHistoryWeight);
	#endif

#else
	// Fast path that uses hardware interpolation
	// This fails to reject based on depth consistently because the depth has been filtered
	float4 HistoryValue = Texture2DSampleLevel(BentNormalHistoryTexture, BentNormalHistorySampler, OldDistanceFieldUVs, 0);
	float PositionWeight = ComputeHistoryWeightBasedOnPosition(UVAndScreenPos.zw, SceneDepth, OldScreenPosition, HistoryValue.w);

	EffectiveHistoryWeight *= PositionWeight;

	OutBentNormal.rgb = lerp(NewValue.rgb, HistoryValue.rgb, EffectiveHistoryWeight);
#endif

	OutBentNormal.rgb = MakeFinite(OutBentNormal.rgb);

	OutBentNormal.a = SceneDepth;

	FLATTEN
	if (UseHistoryFilter > 0)
	{
		// Sign bit of alpha stores whether the history was rejected or not, to be read by the history filter pass
		OutBentNormal.a *= EffectiveHistoryWeight > 0 ? 1 : -1;
	}
}

#define HALF_HISTORY_FILL_KERNEL_SIZE 2
float2 BentNormalAOTexelSize;
float2 MaxSampleBufferUV;

/** Seeds newly rejected history values (which are sources of temporal instability) with the results of a spatial search from stable history values */
void FilterHistoryPS(
	in float4 UVAndScreenPos : TEXCOORD0
	,out float4 OutBentNormal : SV_Target0
	)
{
	float2 BufferUV = UVAndScreenPos.xy;

	#if MANUALLY_CLAMP_UV
		BufferUV = min(BufferUV, MaxSampleBufferUV);
	#endif

	float4 HistoryValue = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, BufferUV, 0);

	// Only do the spatial search for pixels who discarded their history value
	if (HistoryValue.w < 0)
	{
		float SceneDepth = abs(HistoryValue.w);

		float4 Accumulation = 0;

		for (float y = -HALF_HISTORY_FILL_KERNEL_SIZE; y <= HALF_HISTORY_FILL_KERNEL_SIZE; y++)
		{
			for (float x = -HALF_HISTORY_FILL_KERNEL_SIZE; x <= HALF_HISTORY_FILL_KERNEL_SIZE; x++)
			{
				float2 SampleBufferUV = UVAndScreenPos.xy + BentNormalAOTexelSize * float2(x, y);

				#if MANUALLY_CLAMP_UV
					// Distance field AO was computed at 0,0 regardless of viewrect min.
					SampleBufferUV = min(SampleBufferUV, MaxSampleBufferUV);
				#endif

				float4 TextureValue = Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, SampleBufferUV, 0);

				float SampleSceneDepth = abs(TextureValue.w);
				float ValidMask = TextureValue.w > 0;

				// Weight by depth to avoid pulling in values of a foreground object
				// This is a careful tradeoff between ghosting behind panning foreground objects and successful spatial searches to reduce flickering
				float DepthWeight = exp2(-1000 * abs(SceneDepth - SampleSceneDepth) / SceneDepth);

				float2 Weight2D = exp2(-abs(float2(x, y) * 10.0f / HALF_HISTORY_FILL_KERNEL_SIZE));
				float ScreenSpaceSpatialWeight = max(Weight2D.x, Weight2D.y);

				float Weight = ValidMask * ScreenSpaceSpatialWeight * DepthWeight;

				Accumulation.rgb += TextureValue.rgb * Weight;
				Accumulation.a += Weight;
			}
		}

		// Only change the history value if the spatial search turned up something applicable
		if (Accumulation.a > 0.01f)
		{
			float InvWeight = 1.0f / Accumulation.a;
			// Construct the history value as if the spatial search result was the previous history, 
			// And the AO we just computed this frame was the new value
			HistoryValue.xyz = lerp(HistoryValue.xyz, Accumulation.xyz * InvWeight, HistoryWeight);
		}
	}

	OutBentNormal = HistoryValue;
	// Remove sign bit so future reprojection interpolation isn't affected
	OutBentNormal.w = abs(OutBentNormal.w);
}

float MinIndirectDiffuseOcclusion;

/** Upsamples the AO results to full resolution using a bilateral filter. */
void AOUpsamplePS(
	in float4 UVAndScreenPos : TEXCOORD0
	,out float4 OutSceneColor : SV_Target0
	)
{
#if SHADING_PATH_MOBILE
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UVAndScreenPos.xy, UVAndScreenPos.zw);
#else
	FGBufferData GBuffer = GetGBufferData(UVAndScreenPos.xy);
#endif
	// The rectangle is drawn according to GBuffer rect,
	// while the bent normal is always offset at (0, 0)
	const float2 BentNormalUV = UVAndScreenPos.xy - View.ViewRectMin.xy * View.BufferSizeAndInvSize.zw;
	float3 BentNormal = UpsampleDFAO(BentNormalUV, GBuffer.Depth, GBuffer.WorldNormal);

#if MODULATE_SCENE_COLOR
	float Visibility = lerp(length(BentNormal), 1.0f, MinIndirectDiffuseOcclusion);
	OutSceneColor = Visibility;
#else
	OutSceneColor = float4(length(BentNormal).xxx, 1.0f);
#endif
}