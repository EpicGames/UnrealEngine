// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldLightingShared.usf
=============================================================================*/

#pragma once

#ifndef THREADGROUP_SIZEX
#define THREADGROUP_SIZEX 1
#endif   

#ifndef THREADGROUP_SIZEY
#define THREADGROUP_SIZEY 1
#endif

#define THREADGROUP_TOTALSIZE (THREADGROUP_SIZEX * THREADGROUP_SIZEY)

#ifndef DOWNSAMPLE_FACTOR
#define DOWNSAMPLE_FACTOR 1
#endif

#ifndef UPDATEOBJECTS_THREADGROUP_SIZE
#define UPDATEOBJECTS_THREADGROUP_SIZE 1
#endif

#ifndef DISTANCEFIELD_PRIMITIVE_TYPE_DEFINED
#define DISTANCEFIELD_PRIMITIVE_TYPE_DEFINED
#define DFPT_SignedDistanceField 0
#define DFPT_HeightField 1
#endif

#ifndef DISTANCEFIELD_PRIMITIVE_TYPE
#define DISTANCEFIELD_PRIMITIVE_TYPE DFPT_SignedDistanceField
#endif

#ifndef OFFSET_DATA_STRUCT
#define OFFSET_DATA_STRUCT (0)
#endif

#ifndef USE_DISTANCE_FIELD_SAMPLER
#define USE_DISTANCE_FIELD_SAMPLER (0)
#endif

// Size of SDF brick in voxels, including padding
float3 DistanceFieldBrickSize;

// Size of SDF brick unique data in voxels
float3 DistanceFieldUniqueDataBrickSize;

// Size of brick atlas, in bricks
uint3 DistanceFieldBrickAtlasSizeInBricks;

// Mask and Log2 sizes used to decompose linear brick index into 3d with shifts and masks
uint3 DistanceFieldBrickAtlasMask;
uint3 DistanceFieldBrickAtlasSizeLog2;

// Size of a voxel in the brick atlas, which is also 1 / AtlasSize.
float3 DistanceFieldBrickAtlasTexelSize;

float3 DistanceFieldBrickAtlasHalfTexelSize;
float3 DistanceFieldBrickOffsetToAtlasUVScale;
float3 DistanceFieldUniqueDataBrickSizeInAtlasTexels;

// Stores ranges of global brick indices for each asset mip. Brick indices be INVALID_BRICK_INDEX.
ByteAddressBuffer DistanceFieldIndirectionTable;
Buffer<float4> DistanceFieldIndirection2Table;
Texture3D<float4> DistanceFieldIndirectionAtlas;

// Brick atlas texture, used with hardware trilinear filtering
Texture3D DistanceFieldBrickTexture;

// Distance Field Asset data specific to a given mip
struct FDFAssetData
{
	// Number of available mip maps
	uint NumMips;

	// 3d size of the mip's indirection table, which is flattened in DistanceFieldIndirectionTable
	uint3 IndirectionDimensions;

	// Offset into DistanceFieldIndirectionTable that was allocated for this mip
	uint IndirectionTableOffset;

	// Transforms the volume texture encoded distance to volume space 
	float2 DistanceFieldToVolumeScaleBias;

	// Transforms a volume space position to virtual indirection space
	float3 VolumeToIndirectionAdd;
	float3 VolumeToIndirectionScale;
};

// Per-scene buffer containing packed Distance Field Asset data
StructuredBuffer<float4> SceneDistanceFieldAssetData;

// Must match C++
#define DF_ASSET_DATA_MIP_STRIDE 3
#define DF_MAX_NUM_MIPS 3
#define DF_ASSET_DATA_STRIDE (DF_ASSET_DATA_MIP_STRIDE * DF_MAX_NUM_MIPS)
#define INVALID_BRICK_INDEX 0xFFFFFFFF
#define INDIRECTION_DIMENSION_MASK 0x3FF // Must match DistanceField::MaxIndirectionDimension

FDFAssetData LoadDFAssetData(StructuredBuffer<float4> AssetDataBuffer, uint Offset)
{
	uint4 Vector0 = asuint(AssetDataBuffer[Offset + 0]);
	float4 Vector1 = AssetDataBuffer[Offset + 1];
	float4 Vector2 = AssetDataBuffer[Offset + 2];

	FDFAssetData Data;
	Data.IndirectionDimensions.x = Vector0.x & INDIRECTION_DIMENSION_MASK;
	Data.IndirectionDimensions.y = (Vector0.x >> 10) & INDIRECTION_DIMENSION_MASK;
	Data.IndirectionDimensions.z = (Vector0.x >> 20) & INDIRECTION_DIMENSION_MASK;
	Data.NumMips = Vector0.x >> 30;
	Data.IndirectionTableOffset = Vector0.y;

	Data.DistanceFieldToVolumeScaleBias = float2(Vector1.w, Vector2.w);
	Data.VolumeToIndirectionScale = Vector1.xyz;
	Data.VolumeToIndirectionAdd = Vector2.xyz;

	return Data;
}

// ReversedMipIndex must be in [0, NumMips - 1], where 0 is the lowest resolution mip that is always present
FDFAssetData LoadDFAssetData(uint AssetIndex, uint ReversedMipIndex)
{
	return LoadDFAssetData(SceneDistanceFieldAssetData, AssetIndex * DF_ASSET_DATA_STRIDE + ReversedMipIndex * DF_ASSET_DATA_MIP_STRIDE);
}

// Loads the mip asset data for the highest resolution mip, which is also the narrowest band, so the distance field sampled with this asset data will have clamped distances.  
// Use DistanceToMeshSurfaceStandalone instead to get both an accurate and unclamped distance field.
FDFAssetData LoadDFAssetDataHighestResolution(uint AssetIndex)
{
	uint NumMips = LoadDFAssetData(AssetIndex, 0).NumMips;
	return LoadDFAssetData(AssetIndex, NumMips - 1);
}

RWBuffer<uint> RWObjectIndirectArguments;
Buffer<uint> ObjectIndirectArguments;

RWStructuredBuffer<uint> RWCulledObjectIndices;
StructuredBuffer<uint> CulledObjectIndices;

uint GetCulledNumObjects()
{
	// IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance
	return ObjectIndirectArguments[1];
}

// In float4's.  Must match equivalent C++ variables.
#define DF_OBJECT_BOUNDS_STRIDE 3
#define DF_OBJECT_DATA_STRIDE 10
#define HEIGHTFIELD_OBJECT_BOUNDS_STRIDE 3
#define HEIGHTFIELD_OBJECT_DATA_STRIDE 7

struct FDFObjectBounds
{
	FDFVector3 Center;	// World space bounds center
	float SphereRadius;	// World space bounding sphere extent
	float3 BoxExtent;	// World space AABB extent
	uint OftenMoving;
	bool bVisible;
	uint bCastShadow;
	bool bIsNaniteMesh;
	uint bEmissiveLightSource;
	bool bAffectIndirectLightingWhileHidden;
};

uint NumSceneObjects;
StructuredBuffer<float4> SceneObjectBounds;

StructuredBuffer<float4> SceneHeightfieldObjectBounds;
uint NumSceneHeightfieldObjects;

SamplerState DistanceFieldSampler;

FDFObjectBounds LoadDFObjectBounds(uint ObjectIndex)
{
	FDFObjectBounds Bounds;

	float4 Vector0 = SceneObjectBounds[ObjectIndex * DF_OBJECT_BOUNDS_STRIDE + 0];
	float3 PositionHigh = Vector0.xyz;

	float4 Vector1 = SceneObjectBounds[ObjectIndex * DF_OBJECT_BOUNDS_STRIDE + 1];
	float3 PositionLow = Vector1.xyz;

	Bounds.Center = MakeDFVector3(PositionHigh, PositionLow);
	Bounds.SphereRadius = Vector1.w;

	float4 Vector2 = SceneObjectBounds[ObjectIndex * DF_OBJECT_BOUNDS_STRIDE + 2];
	Bounds.BoxExtent = Vector2.xyz;

	uint Flags = asuint(Vector2.w);
	Bounds.OftenMoving = Flags & 1U;
	Bounds.bCastShadow = (Flags & 2U) != 0U;
	Bounds.bIsNaniteMesh = (Flags & 4U) != 0U;
	Bounds.bEmissiveLightSource = (Flags & 8U) != 0U;
	Bounds.bVisible = (Flags & 16U) != 0U;
	Bounds.bAffectIndirectLightingWhileHidden = (Flags & 32U) != 0U;

	return Bounds;
}

struct FHeightfieldObjectBounds
{
	FDFVector3 BoxOrigin;	// World space AABB center
	float3 BoxExtent;		// World space AABB extent
	bool bInAtlas;
};

// Must match FLumenHeightfieldData as heightfield culling is reused
FHeightfieldObjectBounds LoadHeightfieldObjectBounds(uint ObjectIndex)
{
	FHeightfieldObjectBounds Bounds;

	float4 Vector0 = SceneHeightfieldObjectBounds[ObjectIndex * HEIGHTFIELD_OBJECT_BOUNDS_STRIDE + 0];
	float4 Vector1 = SceneHeightfieldObjectBounds[ObjectIndex * HEIGHTFIELD_OBJECT_BOUNDS_STRIDE + 1];

	float3 PositionHigh = Vector0.xyz;
	float3 RelativeWorldPosition = Vector1.xyz;

	Bounds.BoxOrigin = DFTwoSum(PositionHigh, RelativeWorldPosition);

	float4 Vector2 = SceneHeightfieldObjectBounds[ObjectIndex * HEIGHTFIELD_OBJECT_BOUNDS_STRIDE + 2];
	Bounds.BoxExtent = Vector2.xyz;

	uint Flags = asuint(Vector2.w);
	Bounds.bInAtlas = Flags & 1U;

	return Bounds;
}

struct FDFObjectData
{
	// Extent of the mesh bounds in volume space
	float3 VolumePositionExtent;

	// Volume space bias and surface expand amount
	float VolumeSurfaceBias;

	// Whether the mesh used two sided materials and hit positions from tracing will be more inaccurate
	bool bMostlyTwoSided;

	// Deprecated, use VolumeToWorldScale instead
	float VolumeScale;

	// Artist specified world space distance to avoid self shadowing on this mesh, usually to prevent incorrect self-intersection from WPO animation
	float SelfShadowBias;

	// For view distance culling to match main view rasterizer
	float2 MinMaxDrawDistance2;

	uint GPUSceneInstanceIndex;

	// Transforms between world space and the volume space that tracing happens in
	FDFInverseMatrix WorldToVolume;
	FDFMatrix VolumeToWorld;

	// Scales along each axis to transform from volume space to world space.  Transform a scaled direction to better handle non-uniform scaling.
	float3 VolumeToWorldScale;

	// Index into SceneDistanceFieldAssetData
	uint AssetIndex;
};

FDFObjectData LoadDFObjectDataFromBuffer(StructuredBuffer<float4> SourceBuffer, uint ObjectIndex)
{
	float3 PositionHigh = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 0].xyz;

	FDFObjectData Data;

	float4 V0 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 1];
	float4 V1 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 2];
	float4 V2 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 3];
	float4x4 RelativeWorldToVolume = transpose(float4x4(V0, V1, V2, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	Data.WorldToVolume = MakeDFInverseMatrix4x3(PositionHigh, RelativeWorldToVolume);

	float4 Vector3 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 4];
	Data.VolumePositionExtent = Vector3.xyz;
	Data.VolumeSurfaceBias = abs(Vector3.w);
	Data.bMostlyTwoSided = Vector3.w < 0.0f;

	float4 Vector4 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 5];
	Data.MinMaxDrawDistance2 = Vector4.xy;
	Data.SelfShadowBias = Vector4.z;
	Data.GPUSceneInstanceIndex = asuint(Vector4.w);
	
	V0 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 6];
	V1 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 7];
	V2 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 8];
	float4x4 VolumeToRelativeWorld = transpose(float4x4(V0, V1, V2, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	Data.VolumeToWorld = MakeDFMatrix(PositionHigh, VolumeToRelativeWorld);

	float4 Vector8 = SourceBuffer[ObjectIndex * DF_OBJECT_DATA_STRIDE + 9];
	Data.VolumeToWorldScale = Vector8.xyz;
	Data.VolumeScale = min(Data.VolumeToWorldScale.x, min(Data.VolumeToWorldScale.y, Data.VolumeToWorldScale.z));
	Data.AssetIndex = asuint(Vector8.w);

	return Data;
}

struct FHeightfieldObjectData
{
	FDFInverseMatrix WorldToLocal;
	float4 SizeScale;
	float4 AtlasUVScaleBias;
	float4 VisibilityAtlasUVScaleBias;
};

FHeightfieldObjectData LoadHeightfieldObjectDataFromBuffer(StructuredBuffer<float4> SourceBuffer, uint ObjectIndex)
{
	FHeightfieldObjectData Data;

	float3 PositionHigh = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 0].xyz;

	float4 V0 = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 1];
	float4 V1 = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 2];
	float4 V2 = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 3];
	float4x4 RelativeWorldToVolume = transpose(float4x4(V0, V1, V2, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	Data.WorldToLocal = MakeDFInverseMatrix4x3(PositionHigh, RelativeWorldToVolume);

	Data.SizeScale = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 4];
	Data.AtlasUVScaleBias = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 5];
	Data.VisibilityAtlasUVScaleBias = SourceBuffer[ObjectIndex * HEIGHTFIELD_OBJECT_DATA_STRIDE + 6];

	return Data;
}

StructuredBuffer<float4> SceneObjectData;
StructuredBuffer<float4> SceneHeightfieldObjectData;

FDFObjectData LoadDFObjectData(uint ObjectIndex)
{
	return LoadDFObjectDataFromBuffer(SceneObjectData, ObjectIndex);
}

FHeightfieldObjectData LoadHeightfieldObjectData(uint ObjectIndex)
{
	return LoadHeightfieldObjectDataFromBuffer(SceneHeightfieldObjectData, ObjectIndex);
}

uint NumConvexHullPlanes;
float4 ViewFrustumConvexHull[6];

bool ViewFrustumIntersectSphere(float3 SphereOrigin, float SphereRadius)
{
	for (uint PlaneIndex = 0; PlaneIndex < NumConvexHullPlanes; PlaneIndex++)
	{
		float4 PlaneData = ViewFrustumConvexHull[PlaneIndex];
		float PlaneDistance = dot(PlaneData.xyz, SphereOrigin) - PlaneData.w;

		if (PlaneDistance > SphereRadius)
		{
			return false;
		}
	}
	
	return true;
}

bool ViewFrustumIntersectAABB(float3 BoxCenter, float3 BoxExtent)
{
	for (uint PlaneIndex = 0; PlaneIndex < NumConvexHullPlanes; PlaneIndex++)
	{
		float4 PlaneData = ViewFrustumConvexHull[PlaneIndex];

		float PlaneDistance = dot(PlaneData.xyz, BoxCenter) - PlaneData.w;
		float ProjectedExtent = dot(BoxExtent, abs(PlaneData.xyz));

		if (PlaneDistance > ProjectedExtent)
		{
			return false;
		}
	}
	
	return true;
}

float SampleDistanceFieldBrickTexture(float3 UV)
{
#if SUPPORTS_INDEPENDENT_SAMPLERS && !USE_DISTANCE_FIELD_SAMPLER
	return Texture3DSampleLevel(DistanceFieldBrickTexture, GlobalBilinearWrappedSampler, UV, 0).x;
#else
	return Texture3DSampleLevel(DistanceFieldBrickTexture, DistanceFieldSampler, UV, 0).x;
#endif
}

// Returns a Volume space signed distance to the mesh's surface
// This distance is clamped to the band of the current mip
float SampleSparseMeshSignedDistanceField(float3 SampleVolumePosition, FDFAssetData DFAssetData)
{
	// Transform the volume space position into the [0, IndirectionDimensions] virtual indirection space of the mesh
	float3 IndirectionPos = SampleVolumePosition * DFAssetData.VolumeToIndirectionScale + DFAssetData.VolumeToIndirectionAdd;

	int3 IndirectionCoord = IndirectionPos;
#if OFFSET_DATA_STRUCT == 0
	uint IndirectionIndex = (IndirectionCoord.z * DFAssetData.IndirectionDimensions.y + IndirectionCoord.y) * DFAssetData.IndirectionDimensions.x + IndirectionCoord.x;
	// Fetch the brick index that covers this region of the virtual UV space
	uint BrickIndex = DistanceFieldIndirectionTable.Load((DFAssetData.IndirectionTableOffset + IndirectionIndex) * 4);
	bool ValidBrick = BrickIndex != INVALID_BRICK_INDEX;
#elif OFFSET_DATA_STRUCT == 1
	uint IndirectionIndex = (IndirectionCoord.z * DFAssetData.IndirectionDimensions.y + IndirectionCoord.y) * DFAssetData.IndirectionDimensions.x + IndirectionCoord.x;
	float4 BrickOffset = DistanceFieldIndirection2Table.Load(DFAssetData.IndirectionTableOffset + IndirectionIndex);
	bool ValidBrick = BrickOffset.w > 0;
#else
	float4 BrickOffset = DistanceFieldIndirectionAtlas.Load(int4(IndirectionCoord, 0));
	bool ValidBrick = BrickOffset.w > 0;
#endif

	float MaxEncodedDistance = DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y;
	float DistanceField = MaxEncodedDistance;

	// If the brick was not stored then it had the maximum encodable distance
	if (ValidBrick)
	{
		// Half voxel border around each brick
		float3 BrickLocalUV = IndirectionPos - IndirectionCoord;

#if OFFSET_DATA_STRUCT == 0
		// Decompose into 3d offset
		float3 BrickOffset = uint3(
			BrickIndex & DistanceFieldBrickAtlasMask.x, 
			(BrickIndex >> DistanceFieldBrickAtlasSizeLog2.x) & DistanceFieldBrickAtlasMask.y, 
			BrickIndex >> (DistanceFieldBrickAtlasSizeLog2.x + DistanceFieldBrickAtlasSizeLog2.y));
#endif

		float3 AtlasUV = BrickOffset.xyz * DistanceFieldBrickOffsetToAtlasUVScale + BrickLocalUV * DistanceFieldUniqueDataBrickSizeInAtlasTexels + DistanceFieldBrickAtlasHalfTexelSize;
		float EncodedDistanceField = SampleDistanceFieldBrickTexture(AtlasUV);

		DistanceField = EncodedDistanceField * DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y;
	}

	return DistanceField;
}

// Returns the Volume space distance to the mesh's surface, sampling only the lowest resolution mip for speed.  Will be inaccurate near the mesh surface.
// Use this variant when querying distance for culling, rather than sphere tracing
// SampleVolumePosition must be within [-VolumePositionExtent, VolumePositionExtent]
float DistanceToMeshSurfaceStandaloneApproximate(float3 SampleVolumePosition, FDFObjectData DFObjectData)
{
	// Hardcode lowest resolution mip
	FDFAssetData DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, 0);
	float DistanceField = SampleSparseMeshSignedDistanceField(SampleVolumePosition, DFAssetMipData);
	return DistanceField;
}

// Returns the Volume space distance to the mesh's surface
// Use this variant when querying distance for culling, rather than sphere tracing
// SampleVolumePosition must be within [-VolumePositionExtent, VolumePositionExtent]
float DistanceToMeshSurfaceStandalone(float3 SampleVolumePosition, FDFObjectData DFObjectData)
{
	uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
	float DistanceField = 0;

	// Start from the lowest resolution mip, continue sampling higher resolution mips if we are near the surface
	for (uint ReversedMipIndex = 0; ReversedMipIndex < NumMips; ReversedMipIndex++)
	{
		FDFAssetData DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
		DistanceField = SampleSparseMeshSignedDistanceField(SampleVolumePosition, DFAssetMipData);

		float MaxEncodedDistance = DFAssetMipData.DistanceFieldToVolumeScaleBias.x + DFAssetMipData.DistanceFieldToVolumeScaleBias.y;

		if (abs(DistanceField) > .25 * MaxEncodedDistance)
		{
			break;
		}
	}

	return DistanceField;
}

// Only valid near the surface
// SampleVolumePosition must be within [-VolumePositionExtent, VolumePositionExtent]
float3 CalculateMeshSDFGradient(float3 SampleVolumePosition, FDFAssetData DFAssetData)
{
	float3 VirtualUVVoxelSize = 1.0f / float3(DistanceFieldUniqueDataBrickSize);
	float3 VolumeSpaceVoxelSize = VirtualUVVoxelSize / DFAssetData.VolumeToIndirectionScale;

	// Mesh distance fields have a 1 voxel border around the mesh bounds and can't bilinear sample further than 0.5 voxel from that border
	float3 VoxelOffset = 0.5f * VolumeSpaceVoxelSize;
	float R = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x + VoxelOffset.x, SampleVolumePosition.y, SampleVolumePosition.z), DFAssetData);
	float L = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x - VoxelOffset.x, SampleVolumePosition.y, SampleVolumePosition.z), DFAssetData);
	float F = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x, SampleVolumePosition.y + VoxelOffset.y, SampleVolumePosition.z), DFAssetData);
	float B = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x, SampleVolumePosition.y - VoxelOffset.y, SampleVolumePosition.z), DFAssetData);
	float U = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x, SampleVolumePosition.y, SampleVolumePosition.z + VoxelOffset.z), DFAssetData);
	float D = SampleSparseMeshSignedDistanceField(float3(SampleVolumePosition.x, SampleVolumePosition.y, SampleVolumePosition.z - VoxelOffset.z), DFAssetData);

	float3 Gradient = float3(R - L, F - B, U - D);
	return Gradient;
}

Texture2D HeightFieldTexture;
Texture2D HFVisibilityTexture;

float SampleHeightFieldAtlas(float2 UV)
{
#if SUPPORTS_INDEPENDENT_SAMPLERS && !USE_DISTANCE_FIELD_SAMPLER
	float4 SampleValue = Texture2DSampleLevel(HeightFieldTexture, GlobalBilinearWrappedSampler, UV, 0);
#else
	float4 SampleValue = Texture2DSampleLevel(HeightFieldTexture, DistanceFieldSampler, UV, 0);
#endif
	return DecodePackedHeight(SampleValue.xy);
}

float SampleHFVisibilityTexture(float2 UV)
{
#if SUPPORTS_INDEPENDENT_SAMPLERS && !USE_DISTANCE_FIELD_SAMPLER
	return Texture2DSampleLevel(HFVisibilityTexture, GlobalBilinearWrappedSampler, UV, 0).r;
#else
	return Texture2DSampleLevel(HFVisibilityTexture, DistanceFieldSampler, UV, 0).r;
#endif
}
