// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldObjectCulling.usf
=============================================================================*/

#include "Common.ush"
#include "ComputeShaderUtils.ush"
#include "DeferredShadingCommon.ush"
#include "DistanceFieldLightingShared.ush"
#include "DistanceFieldAOShared.ush"
#include "DistanceField/GlobalDistanceFieldShared.ush"

uint ObjectBoundingGeometryIndexCount;

groupshared uint NumGroupObjects;

groupshared uint GroupBaseIndex;
groupshared uint GroupObjectIndices[UPDATEOBJECTS_THREADGROUP_SIZE];

[numthreads(UPDATEOBJECTS_THREADGROUP_SIZE, 1, 1)]
void CullObjectsForViewCS(
	uint GroupIndex : SV_GroupIndex,
	uint3 GroupId : SV_GroupID)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, UPDATEOBJECTS_THREADGROUP_SIZE);
	const uint ObjectIndex = ThreadIndex;

#define USE_FRUSTUM_CULLING 1
#if USE_FRUSTUM_CULLING
	if (ThreadIndex == 0)
	{
		// RWObjectIndirectArguments is zeroed by a clear before this shader, only need to set things that are non-zero (and are not read by this shader as that would be a race condition)
		// IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance
		RWObjectIndirectArguments[0] = ObjectBoundingGeometryIndexCount;
	}

	if (GroupIndex == 0)
	{
		NumGroupObjects = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	if (ObjectIndex < NumSceneObjects)
	{
		uint SourceIndex = ObjectIndex;

		FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
		const float3 TranslatedCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PrimaryView.PreViewTranslation);

		float DistanceToViewSq = GetDistanceToCameraFromViewVectorSqr(PrimaryView.TranslatedWorldCameraOrigin - TranslatedCenter);

		if (DistanceToViewSq < Square(AOMaxViewDistance + DFObjectBounds.SphereRadius)
			&& ViewFrustumIntersectSphere(TranslatedCenter, DFObjectBounds.SphereRadius + AOObjectMaxDistance))
		{
			FDFObjectData DFObjectData = LoadDFObjectData(SourceIndex);

            if ((DFObjectData.MinMaxDrawDistance2.x < 0.0001 || DistanceToViewSq > DFObjectData.MinMaxDrawDistance2.x)
                && (DFObjectData.MinMaxDrawDistance2.y < 0.0001 || DistanceToViewSq < DFObjectData.MinMaxDrawDistance2.y))
            {
                uint DestIndex;
                InterlockedAdd(NumGroupObjects, 1U, DestIndex);
                GroupObjectIndices[DestIndex] = SourceIndex;
            }
        }
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupIndex == 0)
	{
		InterlockedAdd(RWObjectIndirectArguments[1], NumGroupObjects, GroupBaseIndex);
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupIndex < NumGroupObjects)
	{
		uint SourceIndex = GroupObjectIndices[GroupIndex];
		uint DestIndex = GroupBaseIndex + GroupIndex;
		RWCulledObjectIndices[DestIndex] = SourceIndex;
	}

#else

	if (ThreadIndex == 0)
	{
		// IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance
		RWObjectIndirectArguments[0] = ObjectBoundingGeometryIndexCount;
		RWObjectIndirectArguments[1] = NumSceneObjects;
	}

	GroupMemoryBarrierWithGroupSync();

	if (ObjectIndex < NumSceneObjects)
	{
		uint SourceIndex = ObjectIndex;
		uint DestIndex = ObjectIndex;

		RWCulledObjectIndices[DestIndex] = SourceIndex;
	}

#endif
}

/** Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ;
groupshared uint IntegerTileMaxZ;

/** Inner Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ2;
groupshared uint IntegerTileMaxZ2;

/** View rect min in xy, max in zw. */
uint4 ViewDimensions;
float2 NumGroups;

RWStructuredBuffer<float4> RWTileConeAxisAndCos;
RWStructuredBuffer<float4> RWTileConeDepthRanges;

/** Builds tile depth ranges and bounding cones. */
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void BuildTileConesMain(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
    uint ThreadIndex = GroupThreadId.y * THREADGROUP_SIZEX + GroupThreadId.x;

	// Sampling from the texture based off of the ViewRect size because the texture is created on a per-view basis
	float2 BaseLevelScreenUV = (DispatchThreadId.xy + float2(.5f, .5f)) * DOWNSAMPLE_FACTOR * View.BufferSizeAndInvSize.zw;
	float SceneDepth = GetDownsampledDepth(BaseLevelScreenUV);

	// Initialize per-tile variables
    if (ThreadIndex == 0) 
	{
        IntegerTileMinZ = 0x7F7FFFFF;     
        IntegerTileMaxZ = 0;
		IntegerTileMinZ2 = 0x7F7FFFFF;  
		IntegerTileMaxZ2 = 0;
    }

    GroupMemoryBarrierWithGroupSync();
    
	// Use shared memory atomics to build the depth bounds for this tile
	// Each thread is assigned to a pixel at this point

	if (SceneDepth < AOMaxViewDistance)
	{
		InterlockedMin(IntegerTileMinZ, asuint(SceneDepth));
		InterlockedMax(IntegerTileMaxZ, asuint(SceneDepth));
	}

    GroupMemoryBarrierWithGroupSync();

    float MinTileZ = asfloat(IntegerTileMinZ);
    float MaxTileZ = asfloat(IntegerTileMaxZ);

	float HalfZ = .5f * (MinTileZ + MaxTileZ);

	// Compute a second min and max Z, clipped by HalfZ, so that we get two depth bounds per tile
	// This results in more conservative tile depth bounds and fewer intersections
	if (SceneDepth >= HalfZ && SceneDepth < AOMaxViewDistance)
	{
		InterlockedMin(IntegerTileMinZ2, asuint(SceneDepth));
	}

	if (SceneDepth <= HalfZ)
	{
		InterlockedMax(IntegerTileMaxZ2, asuint(SceneDepth));
	}

	GroupMemoryBarrierWithGroupSync();
	
	float MinTileZ2 = asfloat(IntegerTileMinZ2);
	float MaxTileZ2 = asfloat(IntegerTileMaxZ2);

	if (ThreadIndex == 0)
	{
		float3 TileConeVertex;
		float3 TileConeAxis;
		float TileConeAngleCos;
		float TileConeAngleSin;
		float4 ConeAxisDepthRanges;

		{
			float2 ViewSize = float2(1 / View.ViewToClip[0][0], 1 / View.ViewToClip[1][1]);
			float3 TileCorner00 = normalize(float3((GroupId.x + 0) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 0) / NumGroups.y * ViewSize.y * 2, 1));
			float3 TileCorner10 = normalize(float3((GroupId.x + 1) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 0) / NumGroups.y * ViewSize.y * 2, 1));
			float3 TileCorner01 = normalize(float3((GroupId.x + 0) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 1) / NumGroups.y * ViewSize.y * 2, 1));
			float3 TileCorner11 = normalize(float3((GroupId.x + 1) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 1) / NumGroups.y * ViewSize.y * 2, 1));

			TileConeAxis = normalize(TileCorner00 + TileCorner10 + TileCorner01 + TileCorner11);
			TileConeAngleCos = dot(TileConeAxis, TileCorner00);
			TileConeAngleSin = sqrt(1 - TileConeAngleCos * TileConeAngleCos);

			float TileConeAngleTan = TileConeAngleSin / TileConeAngleCos; 
			float ConeExpandDistance = 0;
			float VertexPullbackLength = ConeExpandDistance / TileConeAngleTan;
			float DistanceToNearPlane = length(TileConeAxis / TileConeAxis.z * View.NearPlane);
			// 1 / cos(AngleBetweenTileCenterAndViewForward)
			float InvCosTileAngle = 1.0f / TileConeAxis.z;
			float ConeAxisDistanceMultiply = InvCosTileAngle;
			float ConeAxisDistanceAdd = VertexPullbackLength + DistanceToNearPlane;
			ConeAxisDepthRanges.x = ConeAxisDistanceMultiply * (MinTileZ - ConeExpandDistance) + ConeAxisDistanceAdd;
			ConeAxisDepthRanges.y = ConeAxisDistanceMultiply * (MaxTileZ2 + ConeExpandDistance) + ConeAxisDistanceAdd;
			ConeAxisDepthRanges.z = ConeAxisDistanceMultiply * (MinTileZ2 - ConeExpandDistance) + ConeAxisDistanceAdd;
			ConeAxisDepthRanges.w = ConeAxisDistanceMultiply * (MaxTileZ + ConeExpandDistance) + ConeAxisDistanceAdd;

			// Pull back cone vertex to contain potential samples
			TileConeVertex = float3(0, 0, 0) - TileConeAxis * VertexPullbackLength;
		}

		uint TileIndex = GroupId.y * NumGroups.x + GroupId.x;
		if (IntegerTileMinZ > IntegerTileMaxZ)
		{
			// Guard against IntegerTileMinZ never getting updated
			RWTileConeAxisAndCos[TileIndex] = float4(0, 0, 0, 1);
			RWTileConeDepthRanges[TileIndex] = 0;
		}
		else
		{
			RWTileConeAxisAndCos[TileIndex] = float4(TileConeAxis, TileConeAngleCos);
			RWTileConeDepthRanges[TileIndex] = ConeAxisDepthRanges;
		}
	}
}


struct FObjectCullVertexOutput
{
	nointerpolation float4 TranslatedPositionAndRadius : TEXCOORD0;
	nointerpolation uint2 ObjectIndexInstanceIndex : TEXCOORD1;
};
 
float ConservativeRadiusScale;

/** Used when culling objects into screenspace tile lists */
void ObjectCullVS(
	float4 InPosition : ATTRIBUTE0,
	uint InstanceIndex : SV_InstanceID,
	out FObjectCullVertexOutput Output,
	out float4 OutPosition : SV_POSITION
	)
{
	const uint ObjectIndex = CulledObjectIndices[InstanceIndex];

	//@todo - implement ConservativelyBoundSphere
	FDFObjectBounds ObjectBounds = LoadDFObjectBounds(ObjectIndex);
	const float3 TranslatedCenter = DFFastToTranslatedWorld(ObjectBounds.Center, PrimaryView.PreViewTranslation);
	//@todo - expand to handle conservative rasterization
	float EffectiveRadius = (ObjectBounds.SphereRadius + AOObjectMaxDistance) * ConservativeRadiusScale;
	float3 TranslatedWorldPosition = InPosition.xyz * EffectiveRadius + TranslatedCenter;
	OutPosition = mul(float4(TranslatedWorldPosition, 1), PrimaryView.TranslatedWorldToClip);
	Output.TranslatedPositionAndRadius.xyz = TranslatedCenter;
	Output.TranslatedPositionAndRadius.w = ObjectBounds.SphereRadius;
	Output.ObjectIndexInstanceIndex = uint2(ObjectIndex, InstanceIndex);
} 

/** Used for object <-> tile culling */
bool IntersectObjectWithConeDepthRange(
	float3 TileConeVertex, 
	float3 TileConeAxis, 
	float TileConeAngleCos, 
	float TileConeAngleSin, 
	float2 ConeDepthRange, 
	float2 ConeAxisDistanceMinMax,  
	uint ObjectIndex)
{
	BRANCH
	if (ConeAxisDistanceMinMax.x > ConeDepthRange.x && ConeAxisDistanceMinMax.y < ConeDepthRange.y)
	{
#define USE_DISTANCE_FIELD_FOR_OBJECT_CULLING 1
#if USE_DISTANCE_FIELD_FOR_OBJECT_CULLING
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
		float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);

		// Use the position halfway between the depth ranges as the center for the bounding sphere of this tile depth range
		float3 ViewTileBoundingSphereCenter = TileConeVertex + TileConeAxis * (.5f * (ConeDepthRange.x + ConeDepthRange.y));
		float3 TranslatedWorldTileBoundingSphereCenter = mul(float4(ViewTileBoundingSphereCenter.xyz, 1), View.ViewToTranslatedWorld).xyz;
		float DistanceAlongAxis = .5f * (ConeDepthRange.y - ConeDepthRange.x);
		float FarDepthDistanceToEdgeOfCone = ConeDepthRange.y * TileConeAngleSin / TileConeAngleCos;
		float TileBoundingSphereRadius = sqrt(DistanceAlongAxis * DistanceAlongAxis + FarDepthDistanceToEdgeOfCone * FarDepthDistanceToEdgeOfCone);

		float3 VolumeTileBoundingSphereCenter = mul(float4(TranslatedWorldTileBoundingSphereCenter, 1), TranslatedWorldToVolume).xyz;
		float BoxDistance = ComputeDistanceFromBoxToPoint(-DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent, VolumeTileBoundingSphereCenter) * DFObjectData.VolumeScale;

		BRANCH
		if (BoxDistance < TileBoundingSphereRadius + AOObjectMaxDistance)
		{
			float3 ClampedSamplePosition = clamp(VolumeTileBoundingSphereCenter, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
			float DistanceToClamped = length(VolumeTileBoundingSphereCenter - ClampedSamplePosition);
			float DistanceToOccluder = (DistanceToMeshSurfaceStandalone(ClampedSamplePosition, DFObjectData) + DistanceToClamped) * DFObjectData.VolumeScale;

			BRANCH
			if (DistanceToOccluder < TileBoundingSphereRadius + AOObjectMaxDistance)
			{
				return true;
			}
		}

#else
		return true;
#endif
	}

	return false;
}

StructuredBuffer<float4> TileConeAxisAndCos;
StructuredBuffer<float4> TileConeDepthRanges;

RWStructuredBuffer<uint> RWNumCulledTilesArray;
RWStructuredBuffer<uint> RWCulledTilesStartOffsetArray;
RWBuffer<uint> RWCulledTileDataArray;

/** Intersects a single object with the tile and adds to the intersection list if needed. */
void ObjectCullPS(
	FObjectCullVertexOutput Input, 
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;
	
	uint2 TilePosition = (uint2)SVPos.xy;
	uint TileIndex = TilePosition.y * NumGroups.x + TilePosition.x;
	float4 ConeAxisAndCos = TileConeAxisAndCos[TileIndex];
	float4 ConeAxisDepthRanges = TileConeDepthRanges[TileIndex];
	float3 TileConeVertex = 0;
	float3 TileConeAxis = ConeAxisAndCos.xyz;
	float TileConeAngleCos = ConeAxisAndCos.w;
	float TileConeAngleSin = sqrt(1 - TileConeAngleCos * TileConeAngleCos);

	float3 TranslatedWorldSphereCenter = Input.TranslatedPositionAndRadius.xyz;
	float SphereRadius = Input.TranslatedPositionAndRadius.w;
	float3 ViewSpaceSphereCenter = mul(float4(TranslatedWorldSphereCenter, 1), View.TranslatedWorldToView).xyz;
	
	// A value of 1 is conservative, but has a huge impact on performance
	float RadiusScale = .5f;

	float4 SphereCenterAndRadius = float4(ViewSpaceSphereCenter, SphereRadius + RadiusScale * AOObjectMaxDistance);

	if (SphereIntersectCone(SphereCenterAndRadius, TileConeVertex, TileConeAxis, TileConeAngleCos, TileConeAngleSin))
	{
		float ConeAxisDistance = dot(SphereCenterAndRadius.xyz - TileConeVertex, TileConeAxis);
		float2 ConeAxisDistanceMinMax = float2(ConeAxisDistance + SphereCenterAndRadius.w, ConeAxisDistance - SphereCenterAndRadius.w);

		const uint ObjectIndex = Input.ObjectIndexInstanceIndex.x;
		const uint InstanceIndex = Input.ObjectIndexInstanceIndex.y;

		bool bTileIntersectsObject = IntersectObjectWithConeDepthRange(TileConeVertex, TileConeAxis, TileConeAngleCos, TileConeAngleSin, ConeAxisDepthRanges.xy, ConeAxisDistanceMinMax, ObjectIndex);

		if (!bTileIntersectsObject)
		{
			bTileIntersectsObject = IntersectObjectWithConeDepthRange(TileConeVertex, TileConeAxis, TileConeAngleCos, TileConeAngleSin, ConeAxisDepthRanges.zw, ConeAxisDistanceMinMax, ObjectIndex);
		}

		if (bTileIntersectsObject)
		{
#if SCATTER_CULLING_COUNT_PASS
			InterlockedAdd(RWNumCulledTilesArray[InstanceIndex], 1);
#else
			uint CulledTileIndex;
			InterlockedAdd(RWNumCulledTilesArray[InstanceIndex], 1, CulledTileIndex);

			uint CulledTileDataStart = CulledTilesStartOffsetArray[InstanceIndex];

			RWCulledTileDataArray[(CulledTileDataStart + CulledTileIndex) * CULLED_TILE_DATA_STRIDE + 0] = TileIndex;
			RWCulledTileDataArray[(CulledTileDataStart + CulledTileIndex) * CULLED_TILE_DATA_STRIDE + 1] = ObjectIndex;
#endif
		}
	}
}

RWBuffer<uint> RWObjectTilesIndirectArguments;
StructuredBuffer<uint> NumCulledTilesArray;

#ifndef COMPUTE_START_OFFSET_GROUP_SIZE
#define COMPUTE_START_OFFSET_GROUP_SIZE 1
#endif

[numthreads(COMPUTE_START_OFFSET_GROUP_SIZE, 1, 1)]
void ComputeCulledTilesStartOffsetCS(
	uint GroupIndex : SV_GroupIndex,
	uint3 GroupId : SV_GroupID)
{
	const uint NumCulledObjects = GetCulledNumObjects();

	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, COMPUTE_START_OFFSET_GROUP_SIZE);
	const uint ObjectIndex = ThreadIndex;

	if (ObjectIndex < NumCulledObjects)
	{
		uint NumIntersectingTiles = NumCulledTilesArray[ObjectIndex];
		uint NumConeTraceThreadGroups = (NumIntersectingTiles + CONE_TRACE_TILES_PER_THREADGROUP - 1) / CONE_TRACE_TILES_PER_THREADGROUP;

		uint StartOffsetThreadGroups;
		InterlockedAdd(RWObjectTilesIndirectArguments[0], NumConeTraceThreadGroups, StartOffsetThreadGroups);
		uint StartOffset = StartOffsetThreadGroups * CONE_TRACE_TILES_PER_THREADGROUP;
		RWCulledTilesStartOffsetArray[ObjectIndex] = StartOffset;

		// Pad remaining entries with INVALID_TILE_INDEX so we can skip computing them in the cone tracing pass
		for (uint PaddingTileIndex = NumIntersectingTiles; PaddingTileIndex < NumConeTraceThreadGroups * CONE_TRACE_TILES_PER_THREADGROUP; PaddingTileIndex++)
		{
			RWCulledTileDataArray[(StartOffset + PaddingTileIndex) * CULLED_TILE_DATA_STRIDE + 0] = INVALID_TILE_INDEX;
			RWCulledTileDataArray[(StartOffset + PaddingTileIndex) * CULLED_TILE_DATA_STRIDE + 1] = ObjectIndex;
		}
	}

	if (ThreadIndex == 0)
	{
		RWObjectTilesIndirectArguments[1] = 1;
		RWObjectTilesIndirectArguments[2] = 1;
	}
}