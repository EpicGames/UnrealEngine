// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "DistanceFieldLightingShared.ush"
#include "DistanceFieldAOShared.ush"
#include "DistanceField/GlobalDistanceFieldShared.ush"
#include "Substrate/Substrate.ush"

// Pass a debug value through the pipeline instead of a bent normal
#define PASS_THROUGH_DEBUG_VALUE 0

/** Computes the distance field normal from the GBuffer. */
void ComputeDistanceFieldNormalPS(
	in float4 UVAndScreenPos : TEXCOORD0, 
	in FStereoPSInput StereoInput,
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	// Sample from the center of the top left full resolution texel
	float2 PixelCoord = floor(SVPos.xy) * DOWNSAMPLE_FACTOR + View.ViewRectMin.xy + .5f;
	float2 ScreenUV = float2(PixelCoord * View.BufferSizeAndInvSize.zw);

	float SceneDepth = CalcSceneDepth(ScreenUV);

#if SUBSTRATE_GBUFFER_FORMAT==1 
	float3 WorldNormal = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0))).WorldNormal;
#else
#if SHADING_PATH_MOBILE
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(ScreenUV, PixelCoord);
#else
	FGBufferData GBuffer = GetGBufferData(ScreenUV);
#endif
	float3 WorldNormal = GBuffer.WorldNormal.xyz;
#endif

	OutColor = EncodeDownsampledGBuffer(WorldNormal, SceneDepth);
}

RWTexture2D<float4> RWDistanceFieldNormal;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void ComputeDistanceFieldNormalCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	float2 PixelCoord = DispatchThreadId.xy * DOWNSAMPLE_FACTOR + View.ViewRectMin.xy + .5f;
	float2 ScreenUV = float2(PixelCoord * View.BufferSizeAndInvSize.zw);

	float SceneDepth = CalcSceneDepth(ScreenUV);

#if SUBSTRATE_GBUFFER_FORMAT==1 
	float3 WorldNormal = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0))).WorldNormal;
#else
#if SHADING_PATH_MOBILE
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(ScreenUV, PixelCoord);
#else
	FGBufferData GBuffer = GetGBufferData(ScreenUV);
#endif
	float3 WorldNormal = GBuffer.WorldNormal.xyz;
#endif

	RWDistanceFieldNormal[DispatchThreadId.xy] = EncodeDownsampledGBuffer(WorldNormal, SceneDepth);
}

Buffer<float4> TileConeDepthRanges;
float TanConeHalfAngle;
RWBuffer<uint> RWScreenGridConeVisibility;
RWBuffer<float> RWConeDepthVisibilityFunction;

void HemisphereConeTraceAgainstTileCulledObjects(uint ObjectIndex, uint OutputBaseIndex, FDFVector3 WorldShadingPosition, float SceneDepth, float3 WorldNormal, float3 TangentX, float3 TangentY)
{
	float MaxWorldStepOffset = GetStepOffset(NUM_CONE_STEPS);
	float InvMaxOcclusionDistance = 1.0f / AOObjectMaxDistance;

#if USE_GLOBAL_DISTANCE_FIELD
	InvMaxOcclusionDistance = 1.0f / AOGlobalMaxOcclusionDistance;
#endif

	FDFObjectBounds ObjectBounds = LoadDFObjectBounds(ObjectIndex);

	const float3 CenterToCamera = DFFastSubtractDemote(ObjectBounds.Center, WorldShadingPosition);
	float ObjectDistanceSq = dot(CenterToCamera, CenterToCamera);

	BRANCH
	// Skip tracing objects with a small projected angle 
	if (ObjectBounds.SphereRadius * ObjectBounds.SphereRadius / ObjectDistanceSq > Square(.25f))
	{
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);

		float3 VolumeShadingPosition = DFMultiplyAffineDemote(WorldShadingPosition, DFObjectData.WorldToVolume);
		float BoxDistance = ComputeDistanceFromBoxToPoint(-DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent, VolumeShadingPosition) * DFObjectData.VolumeScale;

		BRANCH
		if (BoxDistance < AOObjectMaxDistance)
		{
			float ObjectOccluderRadius = length(DFObjectData.VolumePositionExtent) * .5f * DFObjectData.VolumeScale;
			float SelfShadowScale = 1.0f / max(DFObjectData.SelfShadowBias, .0001f);

			const uint MaxMipIndex = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips - 1;

#if SDF_TRACING_TRAVERSE_MIPS
			uint ReversedMipIndex = MaxMipIndex;
#else
			// Always trace against medium resolution mip
			uint ReversedMipIndex = min(1u, MaxMipIndex);
#endif
			FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);

			LOOP
			for (uint ConeIndex = 0; ConeIndex < NUM_CONE_DIRECTIONS; ConeIndex++)
			{
				float3 ConeDirection = SampleDirections[ConeIndex].xyz;
				float3 RotatedConeDirection = ConeDirection.x * TangentX + ConeDirection.y * TangentY + ConeDirection.z * WorldNormal;
				float3 ScaledLocalConeDirection = DFMultiplyVector(RotatedConeDirection, DFObjectData.WorldToVolume);
		
				float MinVisibility = 1;
				float WorldStepOffset = GetStepOffset(0);

				#if USE_GLOBAL_DISTANCE_FIELD
					WorldStepOffset += 2;
				#endif

				float CurrentStepOffset = WorldStepOffset;

				LOOP
				for (uint StepIndex = 0; StepIndex < NUM_CONE_STEPS && WorldStepOffset < MaxWorldStepOffset; StepIndex++)
				{
					float3 StepSamplePosition = VolumeShadingPosition + ScaledLocalConeDirection * WorldStepOffset;
					float3 ClampedSamplePosition = fastClamp(StepSamplePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
					float DistanceField = SampleSparseMeshSignedDistanceField(ClampedSamplePosition, DFAssetData);

#if SDF_TRACING_TRAVERSE_MIPS
					float MaxEncodedDistance = DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y;

					// We reached the maximum distance of this mip's narrow band, use a lower resolution mip for next iteration
					if (abs(DistanceField) > MaxEncodedDistance && ReversedMipIndex > 0)
					{
						ReversedMipIndex--;
						DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
					}
					// We are close to the surface, step back to safety and use a higher resolution mip for next iteration
					else if (abs(DistanceField) < .25f * MaxEncodedDistance && ReversedMipIndex < MaxMipIndex)
					{
						DistanceField -= 6.0f * DFObjectData.VolumeSurfaceBias;
						ReversedMipIndex++;
						DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
					}
#endif

					float DistanceToClamped = lengthFast(StepSamplePosition - ClampedSamplePosition);
					float DistanceToOccluder = (DistanceField + DistanceToClamped) * DFObjectData.VolumeScale;

					float SphereRadius = WorldStepOffset * TanConeHalfAngle;
					float InvSphereRadius = rcpFast(SphereRadius);

					// Derive visibility from 1d intersection
					float Visibility = saturate(DistanceToOccluder * InvSphereRadius);

					// Don't allow small objects to fully occlude a cone step
					float SmallObjectVisibility = 1 - saturate(ObjectOccluderRadius * InvSphereRadius);

					// Don't allow occlusion within an object's self shadow distance
					float SelfShadowVisibility = 1 - saturate(WorldStepOffset * SelfShadowScale);
					
					// Fade out occlusion based on distance to occluder to avoid a discontinuity at the max AO distance
					float OccluderDistanceFraction = (WorldStepOffset + DistanceToOccluder) * InvMaxOcclusionDistance;
					float DistanceFadeout = saturate(OccluderDistanceFraction * OccluderDistanceFraction * .6f);

					Visibility = max(Visibility, max(SmallObjectVisibility, max(SelfShadowVisibility, DistanceFadeout)));
					MinVisibility = min(Visibility, MinVisibility);
					
					float MinStepScale = .6f;

					#if USE_GLOBAL_DISTANCE_FIELD
						MinStepScale = 2;
					#endif

					float NextStepOffset = GetStepOffset(StepIndex + 1);
					float MinStepSize = MinStepScale * (NextStepOffset - CurrentStepOffset);
					CurrentStepOffset = NextStepOffset;
					WorldStepOffset += max(DistanceToOccluder, MinStepSize);
				}

#if !PASS_THROUGH_DEBUG_VALUE
				InterlockedMin(RWScreenGridConeVisibility[ConeIndex * ScreenGridConeVisibilitySize.x * ScreenGridConeVisibilitySize.y + OutputBaseIndex], asuint(MinVisibility));
#endif
			}
		}
	}
}

Buffer<uint> CulledTileDataArray;

/** Traces cones of a hemisphere against intersecting object distance fields. */
[numthreads(CONE_TRACE_OBJECTS_THREADGROUP_SIZE, 1, 1)]
void ConeTraceObjectOcclusionCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint PixelIndex = GroupThreadId.x % (CONE_TILE_SIZEX * CONE_TILE_SIZEX);
	uint SubCulledTileIndex = GroupThreadId.x / (CONE_TILE_SIZEX * CONE_TILE_SIZEX);

	uint CulledTileDataBaseIndex = GroupId.x * CONE_TRACE_TILES_PER_THREADGROUP;
	uint CulledTileIndex = CulledTileDataBaseIndex + SubCulledTileIndex;
	uint TileIndex = CulledTileDataArray[CulledTileIndex * CULLED_TILE_DATA_STRIDE + 0];
	uint ObjectIndex = CulledTileDataArray[CulledTileDataBaseIndex * CULLED_TILE_DATA_STRIDE + 1];
	uint2 TileCoordinate = uint2(TileIndex % TileListGroupSize.x, TileIndex / TileListGroupSize.x);
	uint2 PixelCoordinate = uint2(PixelIndex % CONE_TILE_SIZEX, PixelIndex / CONE_TILE_SIZEX);
	uint2 OutputCoordinate = TileCoordinate * CONE_TILE_SIZEX + PixelCoordinate;

	if (TileIndex != INVALID_TILE_INDEX && all(OutputCoordinate < ScreenGridConeVisibilitySize))
	{
		float2 BaseLevelScreenUV = GetBaseLevelScreenUVFromScreenGrid(OutputCoordinate);
		uint OutputBaseIndex = OutputCoordinate.y * ScreenGridConeVisibilitySize.x + OutputCoordinate.x;

		float3 WorldNormal;
		float SceneDepth;
		GetDownsampledGBuffer(BaseLevelScreenUV, WorldNormal, SceneDepth);

		float3 TangentX;
		float3 TangentY;
		FindBestAxisVectors2(WorldNormal, TangentX, TangentY);

		{
			float2 ScreenUV = GetScreenUVFromScreenGrid(OutputCoordinate);
			float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
		
			FDFVector3 WorldShadingPosition = DFMultiply(float3(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth), PrimaryView.ScreenToWorld);

			HemisphereConeTraceAgainstTileCulledObjects(ObjectIndex, OutputBaseIndex, WorldShadingPosition, SceneDepth, WorldNormal, TangentX, TangentY);
		}

#if PASS_THROUGH_DEBUG_VALUE
		// Just increment for every tile / object intersection
		InterlockedAdd(RWScreenGridConeVisibility[OutputBaseIndex + 0], 1);
#endif
	}
}

void HemisphereConeTraceAgainstGlobalDistanceFieldClipmap(
	uniform uint ClipmapIndex,
	uint OutputBaseIndex,
	float3 TranslatedWorldPosition,
	float SceneDepth,
	float3 WorldNormal,
	float3 TangentX,
	float3 TangentY)
{
	const float VoxelExtent = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * GlobalVolumeTexelSize;
	const float MinStepSize = VoxelExtent;
	const float InvAOGlobalMaxOcclusionDistance = 1.0f / AOGlobalMaxOcclusionDistance;

	float InitialOffset = VoxelExtent * 2.0f;
	float ConeTraceLeakFill = 1.0f;

#if CONE_TRACE_OBJECTS
	InitialOffset = max(InitialOffset, GetStepOffset(NUM_CONE_STEPS));
#else
	// Cover AO leaking by comparing initial step SDF value against initial step size.
	{
		float3 TranslatedWorldSamplePosition = TranslatedWorldPosition + WorldNormal * InitialOffset;
		float DistanceToOccluder = SampleGlobalDistanceField(TranslatedWorldSamplePosition, AOGlobalMaxOcclusionDistance, ClipmapIndex);
		ConeTraceLeakFill = saturate(Pow2(DistanceToOccluder / InitialOffset)) * 0.4f + 0.6f;
	}
#endif

	LOOP
	for (uint ConeIndex = 0; ConeIndex < NUM_CONE_DIRECTIONS; ConeIndex++)
	{
		float3 ConeDirection = SampleDirections[ConeIndex].xyz;
		float3 RotatedConeDirection = ConeDirection.x * TangentX + ConeDirection.y * TangentY + ConeDirection.z * WorldNormal;

		float MinVisibility = 1;
		float WorldStepOffset = InitialOffset;

		LOOP
		for (uint StepIndex = 0; StepIndex < NUM_CONE_STEPS && WorldStepOffset < AOGlobalMaxOcclusionDistance; StepIndex++)
		{
			float3 TranslatedWorldSamplePosition = TranslatedWorldPosition + RotatedConeDirection * WorldStepOffset;
			float DistanceToOccluder = SampleGlobalDistanceField(TranslatedWorldSamplePosition, AOGlobalMaxOcclusionDistance, ClipmapIndex);
			float SphereRadius = WorldStepOffset * TanConeHalfAngle;
			float InvSphereRadius = rcpFast(SphereRadius);

			// Derive visibility from 1d intersection
			float Visibility = saturate(DistanceToOccluder * InvSphereRadius);
			
			float OccluderDistanceFraction = (WorldStepOffset + DistanceToOccluder) * InvAOGlobalMaxOcclusionDistance;

			// Fade out occlusion based on distance to occluder to avoid a discontinuity at the max AO distance
			Visibility = max(Visibility, saturate(OccluderDistanceFraction * OccluderDistanceFraction * 0.6f));
			
			MinVisibility = min(MinVisibility, Visibility);

			WorldStepOffset += max(DistanceToOccluder, MinStepSize);
		}

		MinVisibility *= ConeTraceLeakFill;

		#if PASS_THROUGH_DEBUG_VALUE
			//InterlockedAdd(RWScreenGridConeVisibility[OutputBaseIndex + 0], 1);
		#else
			InterlockedMin(RWScreenGridConeVisibility[ConeIndex * ScreenGridConeVisibilitySize.x * ScreenGridConeVisibilitySize.y + OutputBaseIndex], asuint(MinVisibility));
		#endif
	}
}

void HemisphereConeTraceAgainstGlobalDistanceField(uint OutputBaseIndex, float3 TranslatedWorldPosition, float SceneDepth, float3 WorldNormal, float3 TangentX, float3 TangentY)
{
	int MinClipmapIndex = -1;

	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ++ClipmapIndex)
	{
		float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldPosition);
		if (DistanceFromClipmap > AOGlobalMaxOcclusionDistance)
		{
			MinClipmapIndex = ClipmapIndex;
			break;
		}
	}

	if (MinClipmapIndex >= 0)
	{
		HemisphereConeTraceAgainstGlobalDistanceFieldClipmap(MinClipmapIndex, OutputBaseIndex, TranslatedWorldPosition, SceneDepth, WorldNormal, TangentX, TangentY);
	}
}

#ifndef CONE_TRACE_GLOBAL_DISPATCH_SIZEX
#define CONE_TRACE_GLOBAL_DISPATCH_SIZEX 1
#endif
 
/** */
[numthreads(CONE_TRACE_GLOBAL_DISPATCH_SIZEX, CONE_TRACE_GLOBAL_DISPATCH_SIZEX, 1)]
void ConeTraceGlobalOcclusionCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint2 OutputCoordinate = DispatchThreadId.xy;
	
	if (all(OutputCoordinate < ScreenGridConeVisibilitySize))
	{
		float2 BaseLevelScreenUV = GetBaseLevelScreenUVFromScreenGrid(OutputCoordinate);

		float3 WorldNormal;
		float SceneDepth;
		GetDownsampledGBuffer(BaseLevelScreenUV, WorldNormal, SceneDepth);

		float3 TangentX;
		float3 TangentY;
		FindBestAxisVectors2(WorldNormal, TangentX, TangentY);

		float2 ScreenUV = GetScreenUVFromScreenGrid(OutputCoordinate);
		float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
		
		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

		uint OutputBaseIndex = OutputCoordinate.y * ScreenGridConeVisibilitySize.x + OutputCoordinate.x;

		HemisphereConeTraceAgainstGlobalDistanceField(OutputBaseIndex, TranslatedWorldPosition, SceneDepth, WorldNormal, TangentX, TangentY);
	}
}

Buffer<uint> ScreenGridConeVisibility;
RWTexture2D<float4> RWDistanceFieldBentNormal;
uint2 ConeBufferMax;
float2 DFNormalBufferUVMax;

#ifndef COMBINE_CONES_SIZEX
#define COMBINE_CONES_SIZEX 1
#endif

/** */
[numthreads(COMBINE_CONES_SIZEX, COMBINE_CONES_SIZEX, 1)]
void CombineConeVisibilityCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint2 InputCoordinate = min(DispatchThreadId.xy, ConeBufferMax);
	uint2 OutputCoordinate = DispatchThreadId.xy;
	float2 BaseLevelScreenUV = GetBaseLevelScreenUVFromScreenGrid(InputCoordinate);
	BaseLevelScreenUV = min(BaseLevelScreenUV, DFNormalBufferUVMax);

	float3 WorldNormal;
	float SceneDepth;
	GetDownsampledGBuffer(BaseLevelScreenUV, WorldNormal, SceneDepth);

#if ROTATE_WITH_TANGENT
	float3 TangentX;
	float3 TangentY;
	FindBestAxisVectors2(WorldNormal, TangentX, TangentY);
#endif

	uint InputBaseIndex = InputCoordinate.y * ScreenGridConeVisibilitySize.x + InputCoordinate.x;

#if PASS_THROUGH_DEBUG_VALUE
	float BufferValue = ScreenGridConeVisibility[InputBaseIndex + 0] - asuint(1.0f);
	float DebugValue = BufferValue / 100.0f;
	RWDistanceFieldBentNormal[OutputCoordinate] = float4(DebugValue.xxx, SceneDepth);
#else
	float3 UnoccludedDirection = 0;
	
	for (uint ConeIndex = 0; ConeIndex < NUM_CONE_DIRECTIONS; ConeIndex++)
	{
		float ConeVisibility = asfloat(ScreenGridConeVisibility[ConeIndex * ScreenGridConeVisibilitySize.x * ScreenGridConeVisibilitySize.y + InputBaseIndex]);
		
		float3 ConeDirection = SampleDirections[ConeIndex].xyz;
#if ROTATE_WITH_TANGENT
		float3 RotatedConeDirection = ConeDirection.x * TangentX + ConeDirection.y * TangentY + ConeDirection.z * WorldNormal;
#else
		float3 RotatedConeDirection = ConeDirection;
#endif
		UnoccludedDirection += ConeVisibility * RotatedConeDirection;
	}

	float InvNumSamples = 1.0f / (float)NUM_CONE_DIRECTIONS;
	float3 BentNormal = UnoccludedDirection * (BentNormalNormalizeFactor * InvNumSamples);

	RWDistanceFieldBentNormal[OutputCoordinate] = float4(BentNormal, SceneDepth);
#endif
}