// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldShadowing.usf
=============================================================================*/

#include "Common.ush"
#include "ComputeShaderUtils.ush"
#include "DeferredShadingCommon.ush"
#include "DistanceFieldLightingShared.ush"
#include "DistanceFieldShadowingShared.ush"
#include "Substrate/Substrate.ush"

#ifdef UPSAMPLE_PASS
#	include "ShadowFactorsUpsampleCommon.ush"
#endif

#define SCATTER_TILE_CULLING (CULLING_TYPE == 0)
#define POINT_LIGHT (CULLING_TYPE == 2)

#if (DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_SignedDistanceField)
#define MAX_TRACE_SPHERE_RADIUS 100
#define SELF_SHADOW_VERTICAL_BIAS 0
#define SELF_SHADOW_VIEW_BIAS 0
#else
#define MAX_TRACE_SPHERE_RADIUS 500
#define SELF_SHADOW_VERTICAL_BIAS 100
#define SELF_SHADOW_VIEW_BIAS 50
#endif

uint ObjectBoundingGeometryIndexCount;
float ObjectExpandScale;

float4 ShadowConvexHull[12];
float4 ShadowBoundingSphere;
uint NumShadowHullPlanes;
uint bDrawNaniteMeshes;
uint bCullHeighfieldsNotInAtlas;

bool ShadowConvexHullIntersectSphere(float3 SphereOrigin, float SphereRadius)
{
	for (uint PlaneIndex = 0; PlaneIndex < NumShadowHullPlanes; PlaneIndex++)
	{
		float4 PlaneData = ShadowConvexHull[PlaneIndex];
		float PlaneDistance = dot(PlaneData.xyz, SphereOrigin) - PlaneData.w;

		if (PlaneDistance > SphereRadius)
		{
			return false;
		}
	}
	
	return true;
}

bool ShadowConvexHullIntersectBox(float3 BoxOrigin, float3 BoxExtent)
{
	for (uint PlaneIndex = 0; PlaneIndex < NumShadowHullPlanes; ++PlaneIndex)
	{
		float4 PlaneData = ShadowConvexHull[PlaneIndex];
		float PlaneDistance = dot(PlaneData.xyz, BoxOrigin) - PlaneData.w;
		float PushOut = dot(abs(PlaneData.xyz), BoxExtent);

		if (PlaneDistance > PushOut)
		{
			return false;
		}
	}

	return true;
}

uint GetNumSceneObjects()
{
#if (DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_SignedDistanceField)
	return NumSceneObjects;
#else
	return NumSceneHeightfieldObjects;
#endif
}

[numthreads(UPDATEOBJECTS_THREADGROUP_SIZE, 1, 1)]
void CullObjectsForShadowCS(
	uint GroupIndex : SV_GroupIndex,
	uint3 GroupId : SV_GroupID)
{
	const uint ThreadIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, UPDATEOBJECTS_THREADGROUP_SIZE);
	const uint ObjectIndex = ThreadIndex;

#define USE_FRUSTUM_CULLING 1
#if USE_FRUSTUM_CULLING

	if (ThreadIndex == 0)
	{
		// RWObjectIndirectArguments is zeroed by a clear before this shader, only need to set things that are non-zero (and are not read by this shader as that would be a race condition)
		// IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance
		RWObjectIndirectArguments[0] = ObjectBoundingGeometryIndexCount;
	}

	GroupMemoryBarrierWithGroupSync();

	if (ObjectIndex < GetNumSceneObjects())
	{
#if (DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_SignedDistanceField)
		const FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
		const float3 TranslatedCenter = DFFastToTranslatedWorld(DFObjectBounds.Center, PrimaryView.PreViewTranslation);

		const float3 CenterToShadowBoundingSphere = ShadowBoundingSphere.xyz - TranslatedCenter;
		const bool bConvexHullIntersect = ShadowBoundingSphere.w == 0 && ShadowConvexHullIntersectSphere(TranslatedCenter, DFObjectBounds.SphereRadius);
		const bool bBoundingSphereIntersect = ShadowBoundingSphere.w > 0 && dot(CenterToShadowBoundingSphere, CenterToShadowBoundingSphere) < Square(ShadowBoundingSphere.w + DFObjectBounds.SphereRadius);

        if (DFObjectBounds.bCastShadow 
			&& (bConvexHullIntersect || bBoundingSphereIntersect)
			&& (bDrawNaniteMeshes || !DFObjectBounds.bIsNaniteMesh))
		{
			FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);

			// Assume ObjectBoundingSphere is located at (0, 0, 0) in local space
			const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
			float ViewDist2 = length2(TranslatedCenter - TranslatedWorldViewOrigin);

            if ((DFObjectData.MinMaxDrawDistance2.x < 0.0001 || ViewDist2 > DFObjectData.MinMaxDrawDistance2.x)
                && (DFObjectData.MinMaxDrawDistance2.y < 0.0001 || ViewDist2 < DFObjectData.MinMaxDrawDistance2.y))
            {
                uint DestIndex;
                InterlockedAdd(RWObjectIndirectArguments[1], 1U, DestIndex);
                RWCulledObjectIndices[DestIndex] = ObjectIndex;
            }
        }
#else
		FHeightfieldObjectBounds Bounds = LoadHeightfieldObjectBounds(ObjectIndex);
		const float3 TranslatedBoxOrigin = DFFastToTranslatedWorld(Bounds.BoxOrigin, PrimaryView.PreViewTranslation);
		if ((Bounds.bInAtlas || !bCullHeighfieldsNotInAtlas) && ShadowConvexHullIntersectBox(TranslatedBoxOrigin, Bounds.BoxExtent))
		{
			uint DestIndex;
			InterlockedAdd(RWObjectIndirectArguments[1], 1U, DestIndex);
			RWCulledObjectIndices[DestIndex] = ObjectIndex;
		}
#endif
	}

#else

	if (ThreadIndex == 0)
	{
		// IndexCount, NumInstances, StartIndex, BaseVertexIndex, FirstInstance
		RWObjectIndirectArguments[0] = ObjectBoundingGeometryIndexCount;
		RWObjectIndirectArguments[1] = GetNumSceneObjects();
	}

	if (ObjectIndex < GetNumSceneObjects())
	{
		RWCulledObjectIndices[ObjectIndex] = ObjectIndex;
	}
#endif
}

RWBuffer<uint> RWNextStartOffset;
RWBuffer<uint> RWShadowTileStartOffsets;

groupshared uint GroupNumIntersectingObjects;
groupshared uint GroupStartOffset;

#ifndef COMPUTE_START_OFFSET_GROUP_SIZE
#define COMPUTE_START_OFFSET_GROUP_SIZE 1
#endif

[numthreads(COMPUTE_START_OFFSET_GROUP_SIZE, COMPUTE_START_OFFSET_GROUP_SIZE, 1)]
void ComputeCulledTilesStartOffsetCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	bool bFirstThreadInGroup = all(GroupThreadId.xy == 0);

	if (bFirstThreadInGroup)
	{
		GroupNumIntersectingObjects = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	uint2 TileCoordinate = DispatchThreadId.xy;
	bool bValidTile = all(TileCoordinate < ShadowTileListGroupSize);
	uint TileIndex = TileCoordinate.y * ShadowTileListGroupSize.x + TileCoordinate.x;
	uint TileStartOffset = 0;

	if (bValidTile)
	{
		uint NumIntersectingObjects = ShadowTileNumCulledObjects[TileIndex];
		InterlockedAdd(GroupNumIntersectingObjects, NumIntersectingObjects, TileStartOffset);
	}

	GroupMemoryBarrierWithGroupSync();

	if (bFirstThreadInGroup)
	{
		InterlockedAdd(RWNextStartOffset[0], GroupNumIntersectingObjects, GroupStartOffset);
	}

	GroupMemoryBarrierWithGroupSync();
	
	if (bValidTile)
	{
		TileStartOffset += GroupStartOffset;
		RWShadowTileStartOffsets[TileIndex] = TileStartOffset;
	}
}

RWBuffer<uint> RWShadowTileNumCulledObjects;
RWBuffer<uint> RWShadowTileArrayData;

struct FShadowObjectCullVertexOutput
{
	nointerpolation uint ObjectIndex : TEXCOORD0;
};

float MinExpandRadius;

/** Used when culling objects into screenspace tile lists */
void ShadowObjectCullVS(
	float4 InPosition : ATTRIBUTE0,
	uint InstanceIndex : SV_InstanceID,
	out FShadowObjectCullVertexOutput Output,
	out float4 OutPosition : SV_POSITION
)
{
	uint ObjectIndex = CulledObjectIndices[InstanceIndex];

#if (DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_SignedDistanceField)
	FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
	float4x4 ObjectOBBToTranslatedWorld = DFFastToTranslatedWorld(DFObjectData.VolumeToWorld, PrimaryView.PreViewTranslation);

	float ObjectScaleX = (1.0f + MinExpandRadius / length(ObjectOBBToTranslatedWorld[0].xyz));
	float ObjectScaleY = (1.0f + MinExpandRadius / length(ObjectOBBToTranslatedWorld[1].xyz));
	float ObjectScaleZ = (1.0f + MinExpandRadius / length(ObjectOBBToTranslatedWorld[2].xyz));

	// This is written without [].xyz *= Scale; to workaround a bad codegen on Switch
	ObjectOBBToTranslatedWorld[0] = float4(ObjectOBBToTranslatedWorld[0].xyz * ObjectScaleX, ObjectOBBToTranslatedWorld[0].w);
	ObjectOBBToTranslatedWorld[1] = float4(ObjectOBBToTranslatedWorld[1].xyz * ObjectScaleY, ObjectOBBToTranslatedWorld[1].w);
	ObjectOBBToTranslatedWorld[2] = float4(ObjectOBBToTranslatedWorld[2].xyz * ObjectScaleZ, ObjectOBBToTranslatedWorld[2].w);

	float3 TranslatedWorldPosition = mul(float4(InPosition.xyz, 1.0f), ObjectOBBToTranslatedWorld).xyz;
#else

	FHeightfieldObjectBounds Bounds = LoadHeightfieldObjectBounds(ObjectIndex);
	float3 BoxTranslatedOrigin = DFFastToTranslatedWorld(Bounds.BoxOrigin, PrimaryView.PreViewTranslation);
	float3 BoxExtent = Bounds.BoxExtent;

	BoxExtent += MinExpandRadius;

	float3 TranslatedWorldPosition = InPosition.xyz * BoxExtent + BoxTranslatedOrigin;
#endif

	OutPosition = mul(float4(TranslatedWorldPosition, 1.0), TranslatedWorldToShadow);

	// Clamp the vertex to the near plane if it is in front of the near plane
	if (OutPosition.z > 1)
	{
		OutPosition.z = 0.999999f;
		OutPosition.w = 1.0f;
	}

	Output.ObjectIndex = ObjectIndex;
} 

void HandleShadowTileObjectIntersection(uint TileIndex, uint ObjectIndex)
{
	#if SCATTER_CULLING_COUNT_PASS
		InterlockedAdd(RWShadowTileNumCulledObjects[TileIndex], 1U);
	#else
		uint ArrayIndex;
		InterlockedAdd(RWShadowTileNumCulledObjects[TileIndex], 1U, ArrayIndex);

		#if COMPACT_CULLED_SHADOW_OBJECTS
			uint StartOffset = ShadowTileStartOffsets[TileIndex];
		#else
			uint StartOffset = TileIndex * ShadowMaxObjectsPerTile;
		#endif
		uint DataIndex = (ArrayIndex + StartOffset);
		RWShadowTileArrayData[DataIndex] = ObjectIndex;
	#endif
}

struct FObjectViewSpaceBox
{
	float3 Min;
	float3 Max;
	float3 XAxis;
	float3 YAxis;
	float3 ZAxis;
};

FObjectViewSpaceBox GetObjectViewSpaceBox(uint ObjectIndex)
{
	float3 MinViewSpacePosition = float3(2000000, 2000000, 2000000);
	float3 MaxViewSpacePosition = float3(-2000000, -2000000, -2000000);
	float3 ViewSpaceBoundsVertices[8];

	for (uint i = 0; i < 8; i++)
	{
		float3 UnitBoxVertex;
		UnitBoxVertex.x = i & 0x1 ? 1.0f : -1.0f;
		UnitBoxVertex.y = i & 0x2 ? 1.0f : -1.0f;
		UnitBoxVertex.z = i & 0x4 ? 1.0f : -1.0f;

#if (DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_SignedDistanceField)
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
		float3 TranslatedWorldBoundsPosition = DFTransformLocalToTranslatedWorld(UnitBoxVertex * DFObjectData.VolumePositionExtent, DFObjectData.VolumeToWorld, PrimaryView.PreViewTranslation).xyz;
#else
		FHeightfieldObjectBounds HeightfieldObjectBounds = LoadHeightfieldObjectBounds(ObjectIndex);
		float3 TranslatedWorldBoundsPosition = UnitBoxVertex * HeightfieldObjectBounds.BoxExtent + DFFastToTranslatedWorld(HeightfieldObjectBounds.BoxOrigin, PrimaryView.PreViewTranslation);
#endif

		float3 ViewSpacePosition = mul(float4(TranslatedWorldBoundsPosition, 1.0f), TranslatedWorldToShadow).xyz;
		MinViewSpacePosition = min(MinViewSpacePosition, ViewSpacePosition);
		MaxViewSpacePosition = max(MaxViewSpacePosition, ViewSpacePosition);
		ViewSpaceBoundsVertices[i] = ViewSpacePosition;
	}

	float3 ObjectXAxis = (ViewSpaceBoundsVertices[1] - ViewSpaceBoundsVertices[0]) / 2.0f;
	float3 ObjectYAxis = (ViewSpaceBoundsVertices[2] - ViewSpaceBoundsVertices[0]) / 2.0f;
	float3 ObjectZAxis = (ViewSpaceBoundsVertices[4] - ViewSpaceBoundsVertices[0]) / 2.0f;

	MinViewSpacePosition.xy -= ObjectExpandScale * MAX_TRACE_SPHERE_RADIUS;
	MaxViewSpacePosition.xy += ObjectExpandScale * MAX_TRACE_SPHERE_RADIUS;

	FObjectViewSpaceBox ViewBox;
	ViewBox.Min = MinViewSpacePosition;
	ViewBox.Max = MaxViewSpacePosition;
	ViewBox.XAxis = ObjectXAxis / max(dot(ObjectXAxis, ObjectXAxis), .0001f);
	ViewBox.YAxis = ObjectYAxis / max(dot(ObjectYAxis, ObjectYAxis), .0001f);
	ViewBox.ZAxis = ObjectZAxis / max(dot(ObjectZAxis, ObjectZAxis), .0001f);
	return ViewBox;
}

/** Intersects a single object with the tile and adds to the intersection list if needed. */
void ShadowObjectCullPS(
	FShadowObjectCullVertexOutput Input, 
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;
	
	uint2 TilePosition = (uint2)SVPos.xy;
	uint TileIndex = TilePosition.y * ShadowTileListGroupSize.x + TilePosition.x;

#define OBJECT_OBB_INTERSECTION 1
#if OBJECT_OBB_INTERSECTION
	float2 TilePositionForCulling = float2(TilePosition.x, ShadowTileListGroupSize.y - 1 - TilePosition.y);

	float3 ShadowTileMin;
	float3 ShadowTileMax;
	ShadowTileMin.xy = (TilePositionForCulling + 0.0f) / (float2)ShadowTileListGroupSize * 2 - 1;
	ShadowTileMax.xy = (TilePositionForCulling + 1.0f) / (float2)ShadowTileListGroupSize * 2 - 1;

	// Extrude toward light to avoid culling objects between the light and the shadow frustum
	ShadowTileMin.z = 0;
	ShadowTileMax.z = 1000;

	FObjectViewSpaceBox ObjectViewSpaceBox = GetObjectViewSpaceBox(Input.ObjectIndex);

	BRANCH
	// Separating axis test on the AABB
	// Note: don't clip by near plane, objects closer to the light can still cast into the frustum
	if (all(ObjectViewSpaceBox.Max > ShadowTileMin) && all(ObjectViewSpaceBox.Min.xy < ShadowTileMax.xy))
	{
		float3 ObjectCenter = .5f * (ObjectViewSpaceBox.Min + ObjectViewSpaceBox.Max);
		float3 MinProjections = 500000;
		float3 MaxProjections = -500000;

		{
			float3 Corners[8];
			Corners[0] = float3(ShadowTileMin.x, ShadowTileMin.y, ShadowTileMin.z);
			Corners[1] = float3(ShadowTileMax.x, ShadowTileMin.y, ShadowTileMin.z);
			Corners[2] = float3(ShadowTileMin.x, ShadowTileMax.y, ShadowTileMin.z);
			Corners[3] = float3(ShadowTileMax.x, ShadowTileMax.y, ShadowTileMin.z);
			Corners[4] = float3(ShadowTileMin.x, ShadowTileMin.y, ShadowTileMax.z);
			Corners[5] = float3(ShadowTileMax.x, ShadowTileMin.y, ShadowTileMax.z);
			Corners[6] = float3(ShadowTileMin.x, ShadowTileMax.y, ShadowTileMax.z);
			Corners[7] = float3(ShadowTileMax.x, ShadowTileMax.y, ShadowTileMax.z);

			float3 ObjectAxisX = ObjectViewSpaceBox.XAxis;
			float3 ObjectAxisY = ObjectViewSpaceBox.YAxis; 
			float3 ObjectAxisZ = ObjectViewSpaceBox.ZAxis;

			UNROLL
			for (int i = 0; i < 8; i++)
			{
				float3 CenterToVertex = Corners[i] - ObjectCenter;
				float3 Projections = float3(dot(CenterToVertex, ObjectAxisX), dot(CenterToVertex, ObjectAxisY), dot(CenterToVertex, ObjectAxisZ));
				MinProjections = min(MinProjections, Projections);
				MaxProjections = max(MaxProjections, Projections);
			}
		}

		BRANCH
		// Separating axis test on the OBB
		if (all(MinProjections < 1) && all(MaxProjections > -1))
		{
			HandleShadowTileObjectIntersection(TileIndex, Input.ObjectIndex);
		}
	}
	
#else
	{
		HandleShadowTileObjectIntersection(TileIndex, Input.ObjectIndex);
	}
#endif
}

RWTexture2D<float2> RWShadowFactors;
float2 NumGroups;

/** From point being shaded toward light, for directional lights. */
float3 LightDirection;
float4 LightTranslatedPositionAndInvRadius;
float LightSourceRadius;
float RayStartOffsetDepthScale;
float3 TanLightAngleAndNormalThreshold;
int4 ScissorRectMinAndSize;

/** Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ;
groupshared uint IntegerTileMaxZ;

/** Inner Min and Max depth for this tile. */
groupshared uint IntegerTileMinZ2;
groupshared uint IntegerTileMaxZ2;

/** Number of objects affecting the tile, after culling. */
groupshared uint TileNumObjects0;
groupshared uint TileNumObjects1;

float4x4 ScreenToView;

void CullObjectsToTileWithGather(
	float SceneDepth, 
	uint ThreadIndex, 
	uint2 GroupId, 
	float TraceDistance, 
	float MinDepth,
	float MaxDepth,
	out uint NumIntersectingObjects, 
	out uint GroupIndex)
{
	// Initialize per-tile variables
    if (ThreadIndex == 0) 
	{
        IntegerTileMinZ = 0x7F7FFFFF;     
        IntegerTileMaxZ = 0;
		IntegerTileMinZ2 = 0x7F7FFFFF;  
		IntegerTileMaxZ2 = 0;
		TileNumObjects0 = 0;
		TileNumObjects1 = 0;
    }

    GroupMemoryBarrierWithGroupSync();
    
	if (SceneDepth > MinDepth && SceneDepth < MaxDepth)
	{
		// Use shared memory atomics to build the depth bounds for this tile
		// Each thread is assigned to a pixel at this point
		//@todo - move depth range computation to a central point where it can be reused by all the frame's tiled deferred passes!
		InterlockedMin(IntegerTileMinZ, asuint(SceneDepth));
		InterlockedMax(IntegerTileMaxZ, asuint(SceneDepth));
	}

    GroupMemoryBarrierWithGroupSync();

    float MinTileZ = asfloat(IntegerTileMinZ);
    float MaxTileZ = asfloat(IntegerTileMaxZ);

	float HalfZ = .5f * (MinTileZ + MaxTileZ);

	// Compute a second min and max Z, clipped by HalfZ, so that we get two depth bounds per tile
	// This results in more conservative tile depth bounds and fewer intersections
	if (SceneDepth >= HalfZ && SceneDepth < MaxDepth)
	{
		InterlockedMin(IntegerTileMinZ2, asuint(SceneDepth));
	}

	if (SceneDepth <= HalfZ && SceneDepth > MinDepth)
	{
		InterlockedMax(IntegerTileMaxZ2, asuint(SceneDepth));
	}

	GroupMemoryBarrierWithGroupSync();
	
	float MinTileZ2 = asfloat(IntegerTileMinZ2);
	float MaxTileZ2 = asfloat(IntegerTileMaxZ2);

	float3 ViewTileMin;
	float3 ViewTileMax;

	float3 ViewTileMin2;
	float3 ViewTileMax2;

	float ExpandRadius = 0;

	// We operate within both a view rect (for multiple side-by-side views) and a scissor rect relative to the view rect (circumscribing the light's attenuation radius on screen)
	float2 TileSize = 2 * ScissorRectMinAndSize.zw / ((float2)View_ViewSizeAndInvSize.xy * NumGroups);
    float2 ScissorRectMin = float2(-1,-1) + 2 * (ScissorRectMinAndSize.xy - View_ViewRectMin.xy) * View_ViewSizeAndInvSize.zw;

    float2 ScreenTileMin = (GroupId.xy * TileSize + ScissorRectMin) * float2(1, -1);
    float2 ScreenTileMax = ((GroupId.xy + 1) * TileSize + ScissorRectMin) * float2(1, -1); 

	// Get the bounding box for this tile in view space
	// Project the corners into view space using both MinZ and MaxZ, and ensure the bounding box contains both results
#if STEREO_RENDERING
	// Stereo rendering has asymmetrical FOVs for each eye so we need to account for off-center projection and use full view matrices when projecting into view space
	ViewTileMin.xy = min(mul(float4(ScreenTileMin * MinTileZ, MinTileZ, 1), ScreenToView).xy, mul(float4(ScreenTileMin * MaxTileZ2, MaxTileZ2, 1), ScreenToView).xy) - ExpandRadius;
	ViewTileMax.xy = max(mul(float4(ScreenTileMax * MinTileZ, MinTileZ, 1), ScreenToView).xy, mul(float4(ScreenTileMax * MaxTileZ2, MaxTileZ2, 1), ScreenToView).xy) + ExpandRadius;
	ViewTileMin2.xy = min(mul(float4(ScreenTileMin * MinTileZ2, MinTileZ2, 1), ScreenToView).xy, mul(float4(ScreenTileMin * MaxTileZ, MaxTileZ, 1), ScreenToView).xy) - ExpandRadius;
	ViewTileMax2.xy = max(mul(float4(ScreenTileMax * MinTileZ2, MinTileZ2, 1), ScreenToView).xy, mul(float4(ScreenTileMax * MaxTileZ, MaxTileZ, 1), ScreenToView).xy) + ExpandRadius;
#else
	// If we can assume centered projection (symmetrical FOV), we can do a fast trig projection into view space using tan(FOV/2)
	// We start with coordinates in screen space ranging from (-1,-1) to (1,1)
	// The (x,y,z) coordinates for a given screen space point (x,y) in view space are (x * z * tan(H_FOV/2), y *  z * tan(V_FOV/2), z)
	float2 TanViewFOV = GetTanHalfFieldOfView();
	ViewTileMin.xy = min(MinTileZ * ScreenTileMin * TanViewFOV, MaxTileZ2 * ScreenTileMin * TanViewFOV) - ExpandRadius;
	ViewTileMax.xy = max(MinTileZ * ScreenTileMax * TanViewFOV, MaxTileZ2 * ScreenTileMax * TanViewFOV) + ExpandRadius;
	ViewTileMin2.xy = min(MinTileZ2 * ScreenTileMin * TanViewFOV, MaxTileZ * ScreenTileMin * TanViewFOV) - ExpandRadius;
	ViewTileMax2.xy = max(MinTileZ2 * ScreenTileMax * TanViewFOV, MaxTileZ * ScreenTileMax * TanViewFOV) + ExpandRadius;
#endif
	ViewTileMin.z = MinTileZ - ExpandRadius;
	ViewTileMax.z = MaxTileZ2 + ExpandRadius;
	ViewTileMin2.z = MinTileZ2 - ExpandRadius;
	ViewTileMax2.z = MaxTileZ + ExpandRadius;

	// Convert the view space bounding box to a world space bounding sphere
	float3 ViewGroup0Center = (ViewTileMax + ViewTileMin) / 2;
	float3 TranslatedWorldGroup0Center = mul(float4(ViewGroup0Center, 1), View.ViewToTranslatedWorld).xyz;
	float Group0BoundingRadius = length(ViewGroup0Center - ViewTileMax);

	float3 ViewGroup1Center = (ViewTileMax2 + ViewTileMin2) / 2;
	float3 TranslatedWorldGroup1Center = mul(float4(ViewGroup1Center, 1), View.ViewToTranslatedWorld).xyz;
	float Group1BoundingRadius = length(ViewGroup1Center - ViewTileMax2);

#if POINT_LIGHT
	float3 LightVector0 = LightTranslatedPositionAndInvRadius.xyz - TranslatedWorldGroup0Center;
	float LightVector0Length = length(LightVector0);
	float3 LightVector1 = LightTranslatedPositionAndInvRadius.xyz - TranslatedWorldGroup1Center;
	float LightVector1Length = length(LightVector1);
	float3 LightDirection0 = LightVector0 / LightVector0Length;
	float3 LightDirection1 = LightVector1 / LightVector1Length;;
	float RayLength0 = LightVector0Length;
	float RayLength1 = LightVector1Length;

	// Don't operate on tiles completely outside of the light's influence
	bool bTileShouldComputeShadowing = LightVector0Length < 1.0f / LightTranslatedPositionAndInvRadius.w + Group0BoundingRadius
		|| LightVector1Length < 1.0f / LightTranslatedPositionAndInvRadius.w + Group1BoundingRadius;
#else
	float3 LightDirection0 = LightDirection;
	float3 LightDirection1 = LightDirection;
	float RayLength0 = TraceDistance;
	float RayLength1 = TraceDistance;

	// Don't operate on tiles completely outside of the [MinDepth, MaxDepth] range
	bool bTileShouldComputeShadowing = MaxTileZ > MinDepth && MinTileZ < MaxDepth;
#endif

	BRANCH
	if (bTileShouldComputeShadowing)
	{
		uint NumCulledObjects = GetCulledNumObjects();

		// Compute per-tile lists of affecting objects through bounds culling
		// Each thread now operates on a sample instead of a pixel
		LOOP
		for (uint IndexInCulledList = ThreadIndex; IndexInCulledList < NumCulledObjects; IndexInCulledList += THREADGROUP_TOTALSIZE)
		{
			const uint ObjectIndex = CulledObjectIndices[IndexInCulledList];
			const FDFObjectBounds Bounds = LoadDFObjectBounds(ObjectIndex);
			const float3 TranslatedBoundsCenter = DFFastToTranslatedWorld(Bounds.Center, PrimaryView.PreViewTranslation);

			BRANCH
			if (RaySegmentHitSphere(TranslatedWorldGroup0Center, LightDirection0, RayLength0, TranslatedBoundsCenter, Bounds.SphereRadius + Group0BoundingRadius))
			{
				uint ListIndex;
				InterlockedAdd(TileNumObjects0, 1U, ListIndex);
				// Don't overwrite on overflow
				ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_OBJECTS - 1));
				IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS * 0 + ListIndex] = ObjectIndex; 
			}

			BRANCH
			if (RaySegmentHitSphere(TranslatedWorldGroup1Center, LightDirection1, RayLength1, TranslatedBoundsCenter, Bounds.SphereRadius + Group1BoundingRadius))
			{
				uint ListIndex;
				InterlockedAdd(TileNumObjects1, 1U, ListIndex);
				// Don't write out of bounds on overflow
				ListIndex = min(ListIndex, (uint)(MAX_INTERSECTING_OBJECTS - 1));
				IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS * 1 + ListIndex] = ObjectIndex; 
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	GroupIndex = SceneDepth > MaxTileZ2 ? 1 : 0;
	NumIntersectingObjects = min(GroupIndex == 0 ? TileNumObjects0 : TileNumObjects1, (uint)MAX_INTERSECTING_OBJECTS);
}

float MinDepth;
float MaxDepth;
uint DownsampleFactor;

float2 InvOutputBufferSize;
Texture2D ShadowFactorsTexture;
SamplerState ShadowFactorsSampler;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void DistanceFieldShadowingCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint ThreadIndex = GroupThreadId.y * THREADGROUP_SIZEX + GroupThreadId.x;

	float2 ScreenUV = float2((DispatchThreadId.xy * DownsampleFactor + ScissorRectMinAndSize.xy + .5f) * View.BufferSizeAndInvSize.zw);
	float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	
	float SceneDepth;
	float FullResFurthestSceneDepth;
	{
		int2 TopLeftPixelPosition = DispatchThreadId.xy * DownsampleFactor + ScissorRectMinAndSize.xy;

		if (DownsampleFactor == 2)
		{
#if CULLING_SUBSAMPLE_DEPTH
			SceneDepth = FullResFurthestSceneDepth = CalcSceneDepth(ScreenUV);
#else
			// cull shadow only if no full-resolution pixel falls within range, preventing edge artifacts
			float4 SceneDepths = GatherDeviceZ(float2(TopLeftPixelPosition + 1) * View.BufferSizeAndInvSize.zw);
			SceneDepth = ConvertFromDeviceZ(SceneDepths.x);
			FullResFurthestSceneDepth = ConvertFromDeviceZ(FarthestDeviceDepth(FarthestDeviceDepth(SceneDepths.x, SceneDepths.y, SceneDepths.z), SceneDepths.w));
#endif
		}
		else
		{
			float SceneDepth00 = LookupDeviceZ(TopLeftPixelPosition);
			SceneDepth = ConvertFromDeviceZ(SceneDepth00);
			FullResFurthestSceneDepth = SceneDepth;
		}
	}

	float3 OpaqueTranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	// Distance for directional lights to trace
	float TraceDistance = TanLightAngleAndNormalThreshold.z;

	uint NumIntersectingObjects = GetCulledNumObjects();
	uint CulledDataParameter = 0;
	bool bShouldComputeShadowing = FullResFurthestSceneDepth > MinDepth && SceneDepth < MaxDepth;

#define USE_CULLING 1
#if USE_CULLING

#if SCATTER_TILE_CULLING

	if (bShouldComputeShadowing)
	{
		GetShadowTileCulledData(OpaqueTranslatedWorldPosition, CulledDataParameter, NumIntersectingObjects);
	}

#else

	CullObjectsToTileWithGather(SceneDepth, ThreadIndex, GroupId.xy, TraceDistance, MinDepth, MaxDepth, NumIntersectingObjects, CulledDataParameter);

#endif
#endif // USE_CULLING

	float Result = 1.0;

#define COMPUTE_SHADOWING 1
#if COMPUTE_SHADOWING
	BRANCH
	if (bShouldComputeShadowing && NumIntersectingObjects > 0)
	{
		// Keeps result from going all the way sharp
		float MinSphereRadius = .4f;
		// Maintain reasonable culling bounds
		float MaxSphereRadius = MAX_TRACE_SPHERE_RADIUS;

		// Reduce shadowing when we are close the ray origin (only used for heightfield)
		float SelfShadowFadeDistance = 100;
		// Mitigate self-shadow caused by discontinuity on heightfield borders (only used for heightfield)
		float SelfShadowVerticalBias = SELF_SHADOW_VERTICAL_BIAS;
		// Mitigate self-shadow on steep terrain surfces
		float SelfShadowViewBias = SELF_SHADOW_VIEW_BIAS;

		OpaqueTranslatedWorldPosition.z += SelfShadowVerticalBias;
		OpaqueTranslatedWorldPosition -= SelfShadowViewBias * View.ViewForward;

		// World space offset along the start of the ray to avoid incorrect self-shadowing
		float RayStartOffset = 2 + RayStartOffsetDepthScale * SceneDepth;

		#if POINT_LIGHT

			float3 LightVector = LightTranslatedPositionAndInvRadius.xyz - OpaqueTranslatedWorldPosition;
			float LightVectorLength = length(LightVector);
			float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition + LightVector / LightVectorLength * RayStartOffset;
			float3 TranslatedWorldRayEnd = LightTranslatedPositionAndInvRadius.xyz;
			float MaxRayTime = LightVectorLength;
			float MaxAngle = tan(10 * PI / 180.0f);
			// Comparing tangents instead of angles, but tangent is always increasing in this range
			float TanLightAngle = min(LightSourceRadius / LightVectorLength, MaxAngle);

		#else

			float3 TranslatedWorldRayStart = OpaqueTranslatedWorldPosition + LightDirection * RayStartOffset;
			float3 TranslatedWorldRayEnd = OpaqueTranslatedWorldPosition + LightDirection * TraceDistance;
			float MaxRayTime = TraceDistance;
			float TanLightAngle = TanLightAngleAndNormalThreshold.x;

		#endif

		#if SCATTER_TILE_CULLING
			bool bUseScatterTileCulling = true;
		#else
			bool bUseScatterTileCulling = false;
		#endif
		
		#if USE_CULLING
			bool bUseCulling = true;
		#else
			bool bUseCulling = false;
		#endif

		bool bIsHeightField = DISTANCEFIELD_PRIMITIVE_TYPE == DFPT_HeightField;

		if (bIsHeightField)
		{
			Result = ShadowRayTraceThroughCulledHeightFieldObjects(
				TranslatedWorldRayStart,
				TranslatedWorldRayEnd,
				TanLightAngle,
				MaxSphereRadius,
				SelfShadowFadeDistance,
				CulledDataParameter,
				NumIntersectingObjects,
				bUseCulling,
				bUseScatterTileCulling);
		}
		else
		{
			float SubsurfaceDensity = 0;
			bool bUseSubsurfaceTransmission = false;

#if !FORWARD_SHADING && DF_SHADOW_QUALITY > 1 && !SHADING_PATH_MOBILE
		#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, ScreenPosition);
			BRANCH
			if (SubstrateSubSurfaceHeaderGetIsValid(SSSHeader))
			{
				SubsurfaceDensity = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
				bUseSubsurfaceTransmission = true;
			}
		#else
			FGBufferData GBufferData = GetGBufferData(ScreenUV);
			BRANCH
			if (IsSubsurfaceModel(GBufferData.ShadingModelID))
			{
				SubsurfaceDensity = SubsurfaceDensityFromOpacity(GBufferData.CustomData.a);
				bUseSubsurfaceTransmission = true;
			}
		#endif
#endif

			Result = ShadowRayTraceThroughCulledObjects(
				TranslatedWorldRayStart, 
				TranslatedWorldRayEnd, 
				MaxRayTime, 
				TanLightAngle, 
				MinSphereRadius, 
				MaxSphereRadius, 
				SubsurfaceDensity,
				CulledDataParameter, 
				NumIntersectingObjects,
				bUseCulling,
				bUseScatterTileCulling,
				bUseSubsurfaceTransmission,
				/*bExpandSurface*/ false);
		}
	}

#if HAS_PREVIOUS_OUTPUT
	if (bShouldComputeShadowing)
	{
#		if PLATFORM_SUPPORTS_TYPED_UAV_LOAD
			float PrevResult = RWShadowFactors[DispatchThreadId.xy].x;
#		else
			float2 PrevResultUV = (DispatchThreadId.xy + 0.5) * InvOutputBufferSize;
			float PrevResult = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, PrevResultUV, 0).x;
#		endif

		Result = min(Result, PrevResult);
	}
#endif

#else
	//Result = bShouldComputeShadowing;
	Result = bShouldComputeShadowing ? NumIntersectingObjects / 256.0f : 0.0f;
#endif

	#if METAL_ES3_1_PROFILE 
		// clamp max depth to avoid #inf
		SceneDepth = min(SceneDepth, 65500.0f);
	#endif
	RWShadowFactors[DispatchThreadId.xy] = float2(Result, SceneDepth);
}

#ifdef UPSAMPLE_PASS

float FadePlaneOffset;
float InvFadePlaneLength;
float NearFadePlaneOffset;
float InvNearFadePlaneLength;

float OneOverDownsampleFactor;
float2 ShadowFactorsUVBilinearMax;
float ShadowFadeFraction;

void DistanceFieldShadowingUpsamplePS(
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float Output;
	float SceneDepth;
	UpsampleShadowFactors(SVPos, ScissorRectMinAndSize, OneOverDownsampleFactor, MinDepth, MaxDepth, ShadowFactorsTexture, ShadowFactorsSampler, ShadowFactorsUVBilinearMax, Output, SceneDepth);

	float FarBlendFactor = 1.0f - saturate((SceneDepth - FadePlaneOffset) * InvFadePlaneLength);
	Output = lerp(1, Output, FarBlendFactor);

	float NearBlendFactor = saturate((SceneDepth - NearFadePlaneOffset) * InvNearFadePlaneLength);
	Output = lerp(1, Output, NearBlendFactor);
	Output = lerp(1, Output, ShadowFadeFraction);

	OutColor = EncodeLightAttenuation(half4(Output, Output, Output, Output));
}

#endif // UPSAMPLE_PASS

#ifdef SHADOW_TILE_VS

Buffer<uint> TileListData;

void ShadowTileVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION)
{
	uint TileData = TileListData[InstanceId.x];
#if PERMUTATION_TILE_TYPE == 1
	const uint2 TileOrigin = UnpackTileCoord12bits(TileData) * WORK_TILE_SIZE;
#else
	const uint2 TileOrigin = UnpackTileCoord16bits(TileData) * WORK_TILE_SIZE;
#endif
	uint2 TileVertex = TileOrigin;
	TileVertex.x += VertexId == 1 || VertexId == 2 || VertexId == 4 ? WORK_TILE_SIZE : 0;
	TileVertex.y += VertexId == 2 || VertexId == 4 || VertexId == 5 ? WORK_TILE_SIZE : 0;

	// View port is set on the view rect. So no offset are needed.
	Position = float4(float2(TileVertex) * View.ViewSizeAndInvSize.zw * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);
}

#endif // SHADOW_TILE_VS
