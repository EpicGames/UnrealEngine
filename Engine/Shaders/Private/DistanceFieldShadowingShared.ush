// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldShadowingShared.usf
=============================================================================*/

#ifndef COMPACT_CULLED_SHADOW_OBJECTS
	#define COMPACT_CULLED_SHADOW_OBJECTS 1
#endif

uint2 ShadowTileListGroupSize;
uint ShadowMaxObjectsPerTile;

uint2 GetShadowTileHead(uint2 TileCoordinate, Buffer<uint> InShadowTileNumCulledObjects, Buffer<uint> InShadowTileStartOffsets)
{
	uint TileIndex = TileCoordinate.y * ShadowTileListGroupSize.x + TileCoordinate.x;

	return uint2(
#if COMPACT_CULLED_SHADOW_OBJECTS
		InShadowTileStartOffsets[TileIndex],
#else
		TileIndex * ShadowMaxObjectsPerTile,
#endif
		InShadowTileNumCulledObjects[TileIndex]); 
}

float4x4 TranslatedWorldToShadow;

void GetShadowTileCulledDataEx(float3 TranslatedWorldPosition, Buffer<uint> InShadowTileNumCulledObjects, Buffer<uint> InShadowTileStartOffsets, out uint CulledDataStart, out uint NumIntersectingObjects)
{
	// Transform into shadow space
	float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPosition, 1), TranslatedWorldToShadow);
	float2 NormalizedShadowPosition = HomogeneousShadowPosition.xy * .5f + .5f;
	NormalizedShadowPosition.y = 1 - NormalizedShadowPosition.y;
	// Quantize the shadow position to get our tile position
	uint2 TilePosition = (uint2)(NormalizedShadowPosition * ShadowTileListGroupSize);
	// Fetch the tile head information
	uint2 TileHead = GetShadowTileHead(TilePosition, InShadowTileNumCulledObjects, InShadowTileStartOffsets);
	CulledDataStart = TileHead.x;
	NumIntersectingObjects = TileHead.y;
}

Buffer<uint> ShadowTileNumCulledObjects;
Buffer<uint> ShadowTileStartOffsets;
Buffer<uint> ShadowTileArrayData;

Buffer<uint> HeightfieldShadowTileNumCulledObjects;
Buffer<uint> HeightfieldShadowTileStartOffsets;
Buffer<uint> HeightfieldShadowTileArrayData;

void GetShadowTileCulledData(float3 TranslatedWorldPosition, out uint CulledDataStart, out uint NumIntersectingObjects)
{
	GetShadowTileCulledDataEx(TranslatedWorldPosition, ShadowTileNumCulledObjects, ShadowTileStartOffsets, CulledDataStart, NumIntersectingObjects);
}

void GetHeightfieldShadowTileCulledData(float3 TranslatedWorldPosition, out uint CulledDataStart, out uint NumIntersectingObjects)
{
	GetShadowTileCulledDataEx(TranslatedWorldPosition, HeightfieldShadowTileNumCulledObjects, HeightfieldShadowTileStartOffsets, CulledDataStart, NumIntersectingObjects);
}

#define MAX_INTERSECTING_OBJECTS 1024
groupshared uint IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS * 2];

float TwoSidedMeshDistanceBiasScale;

float ShadowRayTraceThroughCulledObjects(
	float3 TranslatedWorldRayStart, 
	float3 TranslatedWorldRayEnd, 
	float MaxRayTime,
	float TanLightAngle, 
	float MinSphereRadius, 
	float MaxSphereRadius, 
	float SubsurfaceDensity,
	uint CulledDataParameter, 
	uint NumIntersectingObjects,
	uniform bool bUseCulling,
	uniform bool bUseScatterTileCulling,
	bool bUseSubsurfaceTransmission,
	bool bExpandSurface)
{
	float MinConeVisibility = 1;
	float3 WorldRayUnitDirection = normalize(TranslatedWorldRayEnd - TranslatedWorldRayStart);

	LOOP
	for (uint ListObjectIndex = 0; ListObjectIndex < NumIntersectingObjects; ListObjectIndex++)
	{
		uint ObjectIndex;

		if (bUseCulling)
		{
			if (bUseScatterTileCulling)
			{
				uint CulledDataStart = CulledDataParameter;
				ObjectIndex = ShadowTileArrayData.Load(ListObjectIndex + CulledDataStart);
			}
			else
			{
				uint GroupIndex = CulledDataParameter;
				ObjectIndex = IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS * GroupIndex + ListObjectIndex];
			}
		}
		else
		{
			ObjectIndex = ListObjectIndex;
		}

		{
			FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
			float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);

			float3 VolumeRayStart = mul(float4(TranslatedWorldRayStart, 1), TranslatedWorldToVolume).xyz;
			float3 VolumeRayEnd = mul(float4(TranslatedWorldRayEnd, 1), TranslatedWorldToVolume).xyz;
			float3 VolumeRayDirection = VolumeRayEnd - VolumeRayStart;
			float VolumeRayLength = length(VolumeRayDirection);
			VolumeRayDirection /= VolumeRayLength;

			// Use max volume scale instead of DFObjectData.VolumeScale which is the min version to avoid over dilating penumbra and flickering during movement
			float WorldToVolumeScale = 1.0f / max3(DFObjectData.VolumeToWorldScale.x, DFObjectData.VolumeToWorldScale.y, DFObjectData.VolumeToWorldScale.z);
			float VolumeMinSphereRadius = MinSphereRadius * WorldToVolumeScale;
			float VolumeMaxSphereRadius = MaxSphereRadius * WorldToVolumeScale;
			float SelfShadowScale = 1.0f / max(DFObjectData.SelfShadowBias * WorldToVolumeScale, .0001f);

			// Expand the intersection box by the radius of the cone at the distance of the object along the cone
			float ObjectCenterDistanceAlongRay = dot(-VolumeRayStart, VolumeRayDirection);
			float LocalConeRadiusAtObject = min(TanLightAngle * max(ObjectCenterDistanceAlongRay, 0), VolumeMaxSphereRadius);

			float2 IntersectionTimes = LineBoxIntersect(VolumeRayStart, VolumeRayEnd, -DFObjectData.VolumePositionExtent - LocalConeRadiusAtObject, DFObjectData.VolumePositionExtent + LocalConeRadiusAtObject);

			BRANCH
			if (IntersectionTimes.x < IntersectionTimes.y)
			{
				FDFAssetData DFAssetData = LoadDFAssetDataHighestResolution(DFObjectData.AssetIndex);

				const uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
				const float ExpandSurfaceDistance = DFObjectData.VolumeSurfaceBias * (exp2(DF_MAX_NUM_MIPS - NumMips) + (DFObjectData.bMostlyTwoSided ? TwoSidedMeshDistanceBiasScale - 1 : 0));
				const float ExpandSurfaceFalloff = 2.0f * ExpandSurfaceDistance;
				
				float MaxEncodedDistance = DFAssetData.DistanceFieldToVolumeScaleBias.x + DFAssetData.DistanceFieldToVolumeScaleBias.y;

				if (bExpandSurface || DFObjectData.bMostlyTwoSided)
				{
					MaxEncodedDistance -= ExpandSurfaceDistance;
				}

				// Prevent incorrect shadowing when sampling invalid bricks by limiting VolumeMaxSphereRadius to MaxEncodedDistance
				VolumeMaxSphereRadius = min(VolumeMaxSphereRadius, MaxEncodedDistance);

				float SampleRayTime = IntersectionTimes.x * VolumeRayLength;
#if DF_SHADOW_QUALITY == 2
				uint MaxSteps = 64;
#elif DF_SHADOW_QUALITY == 1
				uint MaxSteps = 32;
#else
				uint MaxSteps = 20;
#endif
				float MinStepSize = 1.0f / (4 * MaxSteps);

				uint StepIndex = 0;

				LOOP
				for (; StepIndex < MaxSteps; StepIndex++)
				{
					float3 SampleVolumePosition = VolumeRayStart + VolumeRayDirection * SampleRayTime;
					float3 ClampedSamplePosition = clamp(SampleVolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
					float DistanceToClamped = length(ClampedSamplePosition - SampleVolumePosition);
					float DistanceField = SampleSparseMeshSignedDistanceField(ClampedSamplePosition, DFAssetData) + DistanceToClamped;

					if (bExpandSurface || DFObjectData.bMostlyTwoSided)
					{
						// Expand the surface to find thin features, but only away from the start of the trace where it won't introduce incorrect self-occlusion
						// This still causes incorrect self-occlusion at grazing angles
						const float ExpandSurfaceAmount = ExpandSurfaceDistance * saturate(SampleRayTime / ExpandSurfaceFalloff);
						DistanceField -= ExpandSurfaceAmount;
					}

					// Don't allow occlusion within an object's self shadow distance
					float SelfShadowVisibility = 1 - saturate(SampleRayTime * SelfShadowScale);

					float SphereRadius = clamp(TanLightAngle * SampleRayTime, VolumeMinSphereRadius, VolumeMaxSphereRadius);
					float StepVisibility = max(saturate(DistanceField / SphereRadius), SelfShadowVisibility);

					if (bUseSubsurfaceTransmission)
					{
						// Determine the distance that the light traveled through the subsurface object
						// This assumes that anything between this subsurface pixel and the light was also a subsurface material
						float Thickness = SampleRayTime * DFObjectData.VolumeScale;
						float SubsurfaceVisibility = saturate(exp(-Thickness * SubsurfaceDensity));

						// Prevent full occlusion in the range that SSS is effective
						// Note: this may cause the trace to travel through negative regions of the distance field
						// It also prevents visibility from ever going to 0
						StepVisibility = max(StepVisibility, SubsurfaceVisibility);
					}

					MinConeVisibility = min(MinConeVisibility, StepVisibility);

					float StepDistance = max(abs(DistanceField), MinStepSize);
					SampleRayTime += StepDistance;

					// Terminate the trace if we are fully occluded or went past the end of the ray
					if (MinConeVisibility < .01f
						|| SampleRayTime > IntersectionTimes.y * VolumeRayLength)
					{
						break;
					}
				}

				// Force to shadowed as we approach max steps
				MinConeVisibility = min(MinConeVisibility, (1 - StepIndex / (float)MaxSteps));
			}
		}

		if (MinConeVisibility < .01f)
		{
			MinConeVisibility = 0.0f;
			break;
		}
	}

	return MinConeVisibility;
}

float RayTraceHeightfieldLocal(
	float3 LocalRayStart,
	float3 LocalRayUnitDirection,
	float StartRayTime,
	float EndRayTime,
	float4 AtlasUVScaleBias,
	float4 VisUVScaleBias,
	float TanLightAngle,
	float LocalSelfShadowScale,
	float LocalMaxSphereRadius)
{
#if DF_SHADOW_QUALITY == 2
	float MaxSteps = 32;
#elif DF_SHADOW_QUALITY == 1
	float MaxSteps = 16;
#else
	float MaxSteps = 8;
#endif

	float StepRayTime = (EndRayTime - StartRayTime) / MaxSteps;
	float SampleRayTime = StartRayTime;
	float MinConeVisibility = 1.0;
	float MinConeVisibility2 = VisUVScaleBias.x > 0.0 ? 1.0 : 0.0;

	for (float StepIndex = 0; StepIndex < MaxSteps; ++StepIndex)
	{
		float3 StepPosition = LocalRayStart + LocalRayUnitDirection * SampleRayTime;
		float2 StepUV = StepPosition.xy * AtlasUVScaleBias.xy + AtlasUVScaleBias.zw;
		float StepHeight = SampleHeightFieldAtlas(StepUV);

		float SelfShadowVisibility = 1 - saturate(SampleRayTime * LocalSelfShadowScale);
		float SphereRadius = clamp(TanLightAngle * SampleRayTime, 0.0001, LocalMaxSphereRadius);

		SampleRayTime += StepRayTime;

		float Distance = StepPosition.z - StepHeight;
		float StepVisibility = max(saturate(Distance / SphereRadius), SelfShadowVisibility);

		MinConeVisibility = min(MinConeVisibility, StepVisibility);

#if DF_SHADOW_QUALITY == 2
		// Compute ray visibility assuming it starts below heightfield. Only needed when the heightfield has hole
		if (VisUVScaleBias.x > 0.0)
		{
			float2 StepVisUV = StepPosition.xy * VisUVScaleBias.xy + VisUVScaleBias.zw;
			float StepVis = SampleHFVisibilityTexture(StepVisUV);
			bool bIsHole = StepVis > 0.0;
			
			// Assuming no more occlusion within this heightfield if the ray goes through a hole
			if (bIsHole && Distance > 0.0)
			{
				break;
			}

			// Ignore this sample if it is beneath a hole
			if (!bIsHole)
			{
				float StepVisibility2 = max(saturate(-Distance / SphereRadius), SelfShadowVisibility);
				MinConeVisibility2 = min(MinConeVisibility2, StepVisibility2);
			}
		}

		if (max(MinConeVisibility, MinConeVisibility2) < 0.01)
		{
			break;
		}
	}

	return max(MinConeVisibility, MinConeVisibility2);
#else
		if (MinConeVisibility < 0.01)
		{
			break;
		}
	}

	return MinConeVisibility;
#endif
}

float ShadowRayTraceThroughCulledHeightFieldObjects(
	float3 TranslatedWorldRayStart,
	float3 TranslatedWorldRayEnd,
	float TanLightAngle,
	float MaxSphereRadius,
	float SelfShadowFadeDistance,
	uint CulledDataParameter,
	uint NumIntersectingObjects,
	uniform bool bUseCulling,
	uniform bool bUseScatterTileCulling)
{
	float MinConeVisibility = 1.0;

	for (uint ListObjectIndex = 0; ListObjectIndex < NumIntersectingObjects; ++ListObjectIndex)
	{
		uint ObjectIndex;

		if (bUseCulling)
		{
			if (bUseScatterTileCulling)
			{
				uint CulledDataStart = CulledDataParameter;
				ObjectIndex = ShadowTileArrayData.Load(ListObjectIndex + CulledDataStart);
			}
			else
			{
				uint GroupIndex = CulledDataParameter;
				ObjectIndex = IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS * GroupIndex + ListObjectIndex];
			}
		}
		else
		{
			ObjectIndex = ListObjectIndex;
		}

		FHeightfieldObjectData HeightfieldObject = LoadHeightfieldObjectData(ObjectIndex);
		float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(HeightfieldObject.WorldToLocal, PrimaryView.PreViewTranslation);
		float2 HeightFieldSize = HeightfieldObject.SizeScale.xy;
		float WorldToLocalScale = HeightfieldObject.SizeScale.z;

		float3 LocalRayStart = mul(float4(TranslatedWorldRayStart, 1.0), TranslatedWorldToLocal).xyz;
		float3 LocalRayEnd = mul(float4(TranslatedWorldRayEnd, 1.0), TranslatedWorldToLocal).xyz;

		float3 LocalBoundsMin = float3(0, 0, DecodePackedHeight(float2(0, 0)));
		float3 LocalBoundsMax = float3(HeightFieldSize, DecodePackedHeight(float2(1, 1)));

		float2 IntersectionNearFar = LineBoxIntersect(LocalRayStart, LocalRayEnd, LocalBoundsMin, LocalBoundsMax);
		bool bValidIntersection = IntersectionNearFar.y > IntersectionNearFar.x;

		if (bValidIntersection)
		{
			float4 AtlasUVScaleBias = HeightfieldObject.AtlasUVScaleBias;
#if DF_SHADOW_QUALITY == 2
			float4 VisUVScaleBias = HeightfieldObject.VisibilityAtlasUVScaleBias;
#else
			float4 VisUVScaleBias = float4(0, 0, 0, 0);
#endif
			float3 LocalRayDirection = LocalRayEnd - LocalRayStart;
			float LocalRayLength = length(LocalRayDirection);
			float3 LocalRayUnitDirection = LocalRayDirection / LocalRayLength;
			float StartRayTime = LocalRayLength * IntersectionNearFar.x;
			float EndRayTime = LocalRayLength * IntersectionNearFar.y;

			float LocalSelfShadowScale = 1.0 / (SelfShadowFadeDistance * WorldToLocalScale);
			float LocalMaxSphereRadius = MaxSphereRadius * WorldToLocalScale;

			float TempVisibility = RayTraceHeightfieldLocal(
				LocalRayStart,
				LocalRayUnitDirection,
				StartRayTime,
				EndRayTime,
				AtlasUVScaleBias,
				VisUVScaleBias,
				TanLightAngle,
				LocalSelfShadowScale,
				LocalMaxSphereRadius);

			MinConeVisibility = min(MinConeVisibility, TempVisibility);

			if (MinConeVisibility < 0.01)
			{
				MinConeVisibility = 0;
				break;
			}
		}
	}

	return MinConeVisibility;
}
	