// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldStreaming.usf
=============================================================================*/

#include "Common.ush"
#include "ComputeShaderUtils.ush"
#include "DistanceFieldLightingShared.ush"

RWTexture3D<UNORM float> RWDistanceFieldBrickAtlas;

#ifdef CopyDistanceFieldAtlasCS

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void CopyDistanceFieldAtlasCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	RWDistanceFieldBrickAtlas[DispatchThreadId] = DistanceFieldBrickTexture[DispatchThreadId].x;
}

#endif

#ifdef ScatterUploadDistanceFieldAtlasCS

Buffer<uint4> BrickUploadCoordinates;
Buffer<float> BrickUploadData;
uint StartBrickIndex;
uint NumBrickUploads;
uint BrickSize;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void ScatterUploadDistanceFieldAtlasCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint BrickOffsetIndex = GroupId.z * THREADGROUP_SIZE / BrickSize;

	if (BrickOffsetIndex < NumBrickUploads)
	{
		uint BrickIndex = StartBrickIndex + BrickOffsetIndex;
		uint3 VoxelCoordinate = DispatchThreadId % BrickSize;
		uint UploadDataReadIndex = BrickIndex * BrickSize * BrickSize * BrickSize + (VoxelCoordinate.z * BrickSize + VoxelCoordinate.y) * BrickSize + VoxelCoordinate.x;
		uint3 BrickAtlasCoordinate = BrickUploadCoordinates[BrickIndex].xyz;
		RWDistanceFieldBrickAtlas[BrickAtlasCoordinate * BrickSize + VoxelCoordinate] = BrickUploadData[UploadDataReadIndex];
	}
}

#endif

RWTexture3D<float4> RWIndirectionAtlas;

#ifdef ScatterUploadDistanceFieldIndirectionAtlasCS

Buffer<uint> IndirectionUploadIndices;
Buffer<float4> IndirectionUploadData;
uint3 IndirectionAtlasSize;
uint NumIndirectionUploads;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ScatterUploadDistanceFieldIndirectionAtlasCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint Index = DispatchThreadId.x;

	if (Index < NumIndirectionUploads)
	{
		uint OutputIndex = IndirectionUploadIndices[Index];
		uint3 AtlasPosition = uint3(
			OutputIndex % IndirectionAtlasSize.x,
			OutputIndex / IndirectionAtlasSize.x % IndirectionAtlasSize.y,
			OutputIndex / IndirectionAtlasSize.x / IndirectionAtlasSize.y);
		RWIndirectionAtlas[AtlasPosition] = IndirectionUploadData[Index];
	}
}
#endif

#ifdef CopyDistanceFieldIndirectionAtlasCS

uint3 IndirectionDimensions;
uint3 SrcPosition;
uint3 DstPosition;
uint NumAssets;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void CopyDistanceFieldIndirectionAtlasCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint Index = DispatchThreadId.x;
	uint3 LocalPosition = uint3(
		Index % IndirectionDimensions.x,
		Index / IndirectionDimensions.x % IndirectionDimensions.y,
		Index / IndirectionDimensions.x / IndirectionDimensions.y % IndirectionDimensions.z);
	uint AssetIndex = Index / IndirectionDimensions.x / IndirectionDimensions.y / IndirectionDimensions.z;

	if (AssetIndex < NumAssets)
	{
		RWIndirectionAtlas[DstPosition + LocalPosition] = DistanceFieldIndirectionAtlas[SrcPosition + LocalPosition];
	}
}

#endif

RWStructuredBuffer<uint> RWDistanceFieldAssetWantedNumMips;
RWStructuredBuffer<uint> RWDistanceFieldAssetStreamingRequests;

#ifdef ComputeDistanceFieldAssetWantedMipsCS

float3 PreViewTranslationHigh;
float3 PreViewTranslationLow;
float3 Mip1WorldTranslatedCenter;
float3 Mip1WorldExtent;
float3 Mip2WorldTranslatedCenter;
float3 Mip2WorldExtent;

float ComputeSquaredDistanceBetweenAABBs(float3 CenterA, float3 ExtentA, float3 CenterB, float3 ExtentB)
{
	float3 AxisDistances = max(abs(CenterB - CenterA) - (ExtentA + ExtentB), 0);
	return dot(AxisDistances, AxisDistances);
}

uint DebugForceNumMips;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ComputeDistanceFieldAssetWantedMipsCS(
	uint GroupIndex : SV_GroupIndex,
	uint3 GroupId : SV_GroupID)
{
	const uint Index = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, THREADGROUP_SIZE);

	if (Index == 0)
	{
		RWDistanceFieldAssetStreamingRequests[0] = 0;
	}

	const uint ObjectIndex = Index;

	if (ObjectIndex < NumSceneObjects)
	{
		uint WantedNumMips = 1;
		if (DebugForceNumMips == 0)
		{
			FDFObjectBounds DFObjectBounds = LoadDFObjectBounds(ObjectIndex);
			FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);
			const float3 TranslatedCenter = DFFastAddDemote(DFObjectBounds.Center, PreViewTranslation);

			float OuterDistanceSq = ComputeSquaredDistanceBetweenAABBs(Mip1WorldTranslatedCenter, Mip1WorldExtent, TranslatedCenter, DFObjectBounds.BoxExtent);
 
			if (OuterDistanceSq <= 0)
			{
				float InnerDistanceSq = ComputeSquaredDistanceBetweenAABBs(Mip2WorldTranslatedCenter, Mip2WorldExtent, TranslatedCenter, DFObjectBounds.BoxExtent);
				WantedNumMips = InnerDistanceSq <= 0 ? 3 : 2;
			}
		}
		else
		{
			WantedNumMips = DebugForceNumMips;
		}

		// RWDistanceFieldAssetWantedNumMips is already cleared to 1 so we can skip some memory loads and atomics
		BRANCH
		if (WantedNumMips > 1)
		{
			FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
			InterlockedMax(RWDistanceFieldAssetWantedNumMips[DFObjectData.AssetIndex], WantedNumMips);
		}
	}
}

#endif

#ifdef GenerateDistanceFieldAssetStreamingRequestsCS

StructuredBuffer<uint> DistanceFieldAssetWantedNumMips;
uint NumDistanceFieldAssets;
uint MaxNumStreamingRequests;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void GenerateDistanceFieldAssetStreamingRequestsCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint AssetIndex = DispatchThreadId.x;

	if (AssetIndex < NumDistanceFieldAssets)
	{
		uint NumMips = LoadDFAssetData(AssetIndex, 0).NumMips;
		uint WantedNumMips = DistanceFieldAssetWantedNumMips[AssetIndex];

		if (WantedNumMips != 0 && NumMips != 0 && NumMips != WantedNumMips)
		{
			uint StreamingRequestIndex;
			InterlockedAdd(RWDistanceFieldAssetStreamingRequests[0], 1, StreamingRequestIndex);

			if (StreamingRequestIndex < MaxNumStreamingRequests)
			{
				RWDistanceFieldAssetStreamingRequests[1 + StreamingRequestIndex * 2 + 0] = AssetIndex;
				RWDistanceFieldAssetStreamingRequests[1 + StreamingRequestIndex * 2 + 1] = WantedNumMips;
			}
		}
	}
}

#endif
