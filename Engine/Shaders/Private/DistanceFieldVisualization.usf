// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "Visualization.ush"
#include "DeferredShadingCommon.ush"
#include "DistanceFieldLightingShared.ush"
#include "DistanceFieldAOShared.ush"
#include "DistanceField/GlobalDistanceFieldShared.ush"
#include "DistanceField/GlobalDistanceFieldUtils.ush"
#include "ReflectionEnvironmentShared.ush"

RWTexture2D<float4> RWVisualizeMeshDistanceFields;

#define MAX_INTERSECTING_OBJECTS 2048
groupshared uint IntersectingObjectIndices[MAX_INTERSECTING_OBJECTS];

groupshared uint NumIntersectingObjects;

#define SDF_TRACING_TRAVERSE_MIPS 1

void RayTraceThroughTileCulledDistanceFields(float3 TranslatedWorldRayStart, float3 WorldRayDirection, float MaxRayTime, out float MinRayTime, out float TotalStepsTaken, out uint HitCulledObjectIndex)
{
	MinRayTime = MaxRayTime;
	TotalStepsTaken = 0;
	HitCulledObjectIndex = 0xFFFFFFFF;

	LOOP
	for (uint ListObjectIndex = 0; ListObjectIndex < min(NumIntersectingObjects, (uint)MAX_INTERSECTING_OBJECTS); ListObjectIndex++)
	{
		uint ObjectIndex = IntersectingObjectIndices[ListObjectIndex];
		FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
		float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);

		float3 VolumeRayStart = mul(float4(TranslatedWorldRayStart, 1.0f), TranslatedWorldToVolume).xyz;
		float3 TranslatedWorldRayEnd = TranslatedWorldRayStart + WorldRayDirection * MaxRayTime;
		float3 VolumeRayEnd = mul(float4(TranslatedWorldRayEnd, 1.0f), TranslatedWorldToVolume).xyz;
		float3 VolumeRayDirection = VolumeRayEnd - VolumeRayStart;
		float VolumeRayLength = length(VolumeRayDirection);
		VolumeRayDirection /= VolumeRayLength;

		float2 VolumeSpaceIntersectionTimes = LineBoxIntersect(VolumeRayStart, VolumeRayEnd, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent) * VolumeRayLength;

		if (VolumeSpaceIntersectionTimes.x < VolumeSpaceIntersectionTimes.y && VolumeSpaceIntersectionTimes.x < VolumeRayLength)
		{
			uint ReversedMipIndex = 0;
			FDFAssetData DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
			uint MaxMipIndex = DFAssetMipData.NumMips - 1;

			#if !SDF_TRACING_TRAVERSE_MIPS
				ReversedMipIndex = MaxMipIndex;
				DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
			#endif

			float SampleRayTime = VolumeSpaceIntersectionTimes.x;
			uint StepIndex = 0;
			uint MaxSteps = 64;
			bool bHit = false;

			LOOP
			for (; StepIndex < MaxSteps; StepIndex++)
			{
				float3 SampleVolumePosition = VolumeRayStart + VolumeRayDirection * SampleRayTime;
				float DistanceField = SampleSparseMeshSignedDistanceField(SampleVolumePosition, DFAssetMipData);

				// Expand the surface to find thin features, but only away from the start of the trace where it won't introduce incorrect self-occlusion
				// This still causes incorrect self-occlusion at grazing angles
				float ExpandSurfaceDistance = DFObjectData.VolumeSurfaceBias;
				const float ExpandSurfaceFalloff = 2.0f * ExpandSurfaceDistance;
				const float ExpandSurfaceAmount = ExpandSurfaceDistance * saturate(SampleRayTime / ExpandSurfaceFalloff);

#if SDF_TRACING_TRAVERSE_MIPS

				float MaxEncodedDistance = DFAssetMipData.DistanceFieldToVolumeScaleBias.x + DFAssetMipData.DistanceFieldToVolumeScaleBias.y;

				if (abs(DistanceField) > MaxEncodedDistance && ReversedMipIndex > 0)
				{
					ReversedMipIndex--;
					DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
				}
				else if (abs(DistanceField) < .25f * MaxEncodedDistance && ReversedMipIndex < MaxMipIndex)
				{
					DistanceField -= 6.0f * DFObjectData.VolumeSurfaceBias;
					ReversedMipIndex++;
					DFAssetMipData = LoadDFAssetData(DFObjectData.AssetIndex, ReversedMipIndex);
				}
				else 
#endif
				// Terminate the trace if we reached a negative area or went past the end of the ray
				if (DistanceField < ExpandSurfaceAmount && ReversedMipIndex == MaxMipIndex)
				{
					bHit = true;
					// One more step back to the surface
					SampleRayTime = clamp(SampleRayTime + DistanceField - ExpandSurfaceAmount, VolumeSpaceIntersectionTimes.x, VolumeSpaceIntersectionTimes.y);
					break;
				}

				float MinStepSize = 1.0f / (16 * MaxSteps);
				float StepDistance = max(DistanceField, MinStepSize);
				SampleRayTime += StepDistance;

				if (SampleRayTime > VolumeSpaceIntersectionTimes.y + ExpandSurfaceAmount)
				{
					break;
				}
			}

			if (bHit || StepIndex == MaxSteps)
			{
				const float NewHitDistance = length(VolumeRayDirection * SampleRayTime * DFObjectData.VolumeToWorldScale);

				if (NewHitDistance < MinRayTime)
				{
					HitCulledObjectIndex = ObjectIndex;
					MinRayTime = NewHitDistance;
				}
			}

			TotalStepsTaken += max(StepIndex, 1u);
		}
	}
}

float2 NumGroups;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void VisualizeMeshDistanceFieldCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint ThreadIndex = GroupThreadId.y * THREADGROUP_SIZEX + GroupThreadId.x;

	float2 ScreenUV = float2((DispatchThreadId.xy * DOWNSAMPLE_FACTOR + View.ViewRectMin.xy + .5f) * View.BufferSizeAndInvSize.zw);
	float2 ScreenPosition = (ScreenUV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	float SceneDepth = CalcSceneDepth(ScreenUV);
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;

	float TraceDistance = 100000.0f;
	float3 TranslatedWorldRayStart = PrimaryView.TranslatedWorldCameraOrigin;
	float3 WorldRayDirection = normalize(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);
	float3 WorldHitNormal = float3(0, 0, 1);
	bool bHit = false;

#if USE_GLOBAL_DISTANCE_FIELD
	FGlobalSDFTraceInput TraceInput = SetupGlobalSDFTraceInput(TranslatedWorldRayStart, WorldRayDirection, 0.0f, TraceDistance, 1.0f, 1.0f);
	FGlobalSDFTraceResult SDFTraceResult = RayTraceGlobalDistanceField(TraceInput);

	if (GlobalSDFTraceResultIsHit(SDFTraceResult))
	{
		float3 TranslatedWorldRayEnd = TranslatedWorldRayStart + WorldRayDirection * SDFTraceResult.HitTime;
		WorldHitNormal = ComputeGlobalDistanceFieldNormal(TranslatedWorldRayEnd, SDFTraceResult.HitClipmapIndex, -WorldRayDirection);
		bHit = true;
	}

#else
	if (ThreadIndex == 0)
	{
		NumIntersectingObjects = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	float3 TileConeTranslatedVertex;
	float3 TileConeAxis;
	float TileConeAngleCos;
	float TileConeAngleSin;

	{
		float2 ViewSize = float2(1 / View.ViewToClip[0][0], 1 / View.ViewToClip[1][1]);
		float3 TileCorner00 = normalize(float3((GroupId.x + 0) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 0) / NumGroups.y * ViewSize.y * 2, 1));
		float3 TileCorner10 = normalize(float3((GroupId.x + 1) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 0) / NumGroups.y * ViewSize.y * 2, 1));
		float3 TileCorner01 = normalize(float3((GroupId.x + 0) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 1) / NumGroups.y * ViewSize.y * 2, 1));
		float3 TileCorner11 = normalize(float3((GroupId.x + 1) / NumGroups.x * ViewSize.x * 2 - ViewSize.x, ViewSize.y - (GroupId.y + 1) / NumGroups.y * ViewSize.y * 2, 1));

		float3 ViewSpaceTileConeAxis = normalize(TileCorner00 + TileCorner10 + TileCorner01 + TileCorner11);
		TileConeAxis = mul(ViewSpaceTileConeAxis, (float3x3)View.ViewToTranslatedWorld);
		TileConeAngleCos = min(min(dot(ViewSpaceTileConeAxis, TileCorner00), dot(ViewSpaceTileConeAxis, TileCorner10)), min(dot(ViewSpaceTileConeAxis, TileCorner01), dot(ViewSpaceTileConeAxis, TileCorner11)));
		TileConeAngleSin = sqrt(1 - TileConeAngleCos * TileConeAngleCos);
		TileConeTranslatedVertex = PrimaryView.TranslatedWorldCameraOrigin;
	}

	uint NumCulledObjects = GetCulledNumObjects();

	LOOP
	for (uint IndexInCulledList = ThreadIndex; IndexInCulledList < NumCulledObjects; IndexInCulledList += THREADGROUP_TOTALSIZE)
	{
		const uint ObjectIndex = CulledObjectIndices[IndexInCulledList];
		FDFObjectBounds ObjectBounds = LoadDFObjectBounds(ObjectIndex);
		float4 SphereTranslatedCenterAndRadius = float4(DFFastAddDemote(ObjectBounds.Center, PrimaryView.PreViewTranslation), ObjectBounds.SphereRadius);

		BRANCH
		if (SphereIntersectCone(SphereTranslatedCenterAndRadius, TileConeTranslatedVertex, TileConeAxis, TileConeAngleCos, TileConeAngleSin))
		{
			uint ListIndex;
			InterlockedAdd(NumIntersectingObjects, 1U, ListIndex);

			if (ListIndex < MAX_INTERSECTING_OBJECTS)
			{
				IntersectingObjectIndices[ListIndex] = ObjectIndex; 
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	float MinRayTime;
	float TotalStepsTaken;
	uint HitCulledObjectIndex;

	// Trace once to find the distance to first intersection
	RayTraceThroughTileCulledDistanceFields(TranslatedWorldRayStart, WorldRayDirection, TraceDistance, MinRayTime, TotalStepsTaken, HitCulledObjectIndex);

	if (HitCulledObjectIndex != 0xFFFFFFFF)
	{
		FDFObjectData DFObjectData = LoadDFObjectData(HitCulledObjectIndex);
		float4x4 TranslatedWorldToVolume = DFFastToTranslatedWorld(DFObjectData.WorldToVolume, PrimaryView.PreViewTranslation);

		float3 TranslatedWorldRayEnd = TranslatedWorldRayStart + WorldRayDirection * MinRayTime;
		float3 VolumeHitPosition = mul(float4(TranslatedWorldRayEnd, 1.0f), TranslatedWorldToVolume).xyz;
		//VolumeHitPosition = clamp(VolumeHitPosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);
		uint NumMips = LoadDFAssetData(DFObjectData.AssetIndex, 0).NumMips;
		FDFAssetData DFAssetData = LoadDFAssetData(DFObjectData.AssetIndex, NumMips - 1);
		float3 VolumeGradient = CalculateMeshSDFGradient(VolumeHitPosition, DFAssetData);
		float VolumeGradientLength = length(VolumeGradient);
		float3 VolumeNormal = VolumeGradientLength > 0.0f ? VolumeGradient / VolumeGradientLength : 0;
		float3 WorldGradient = mul(VolumeNormal, transpose((float3x3)DFObjectData.WorldToVolume.M));
		float WorldGradientLength = length(WorldGradient);
		WorldHitNormal = WorldGradientLength > 0.0f ? WorldGradient / WorldGradientLength : 0;
		bHit = true;
	}
#endif

	float3 Lighting = 0;

	float3 DirectionalLightColor = 10;
	float3 DirectionalLightDirection = float3(.707f, 0, .707f);

	if (ForwardLightStruct.HasDirectionalLight)
	{
		DirectionalLightColor = ForwardLightStruct.DirectionalLightColor;
		DirectionalLightDirection = ForwardLightStruct.DirectionalLightDirection;
	}

	if (bHit)
	{
		// Compute simple lighting to help visualize errors in either position or normal in a way that is intuitive to artists
		Lighting += DirectionalLightColor * saturate(dot(DirectionalLightDirection, WorldHitNormal));

		Lighting += GetSkySHDiffuse(WorldHitNormal) * View.SkyLightColor.rgb;
		float3 Albedo = .05f;
#if !USE_GLOBAL_DISTANCE_FIELD && !MONOCHROME_VISUALIZATION
		{
			FDFObjectData DFObjectData = LoadDFObjectData(HitCulledObjectIndex);
			Albedo = 0.1f * IntToColor(DFObjectData.GPUSceneInstanceIndex + 1);
		}
#endif
		Lighting *= Albedo;
	}
	else if (ReflectionStruct.SkyLightParameters.y > 0)
	{
		float SkyAverageBrightness = 1.0f;
		float Roughness = 0.0f;
		Lighting += GetSkyLightReflection(WorldRayDirection, Roughness, SkyAverageBrightness);
	}

	float3 Result = Lighting * View.PreExposure;

	RWVisualizeMeshDistanceFields[DispatchThreadId.xy + View.ViewRectMin.xy / DOWNSAMPLE_FACTOR] = float4(Result, 0);
}

Texture2D VisualizeDistanceFieldTexture;
SamplerState VisualizeDistanceFieldSampler;

void VisualizeDistanceFieldUpsamplePS(in float4 UVAndScreenPos : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	float3 Value = Texture2DSampleLevel(VisualizeDistanceFieldTexture, VisualizeDistanceFieldSampler, UVAndScreenPos.xy, 0).xyz;

	OutColor = float4(Value, 1);
}
