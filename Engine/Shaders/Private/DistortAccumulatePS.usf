// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionAccumulatePixelShader.usf: Pixel shader for accumulating distortion offsets
=============================================================================*/

#include "Common.ush"

// Reroute distortion pass uniform buffer. 
#if SHADING_PATH_MOBILE
	#define MobileSceneTextures MobileDistortionPass.SceneTextures
	#define DistortionParams MobileDistortionPass.DistortionParams
	#define SubstrateStruct MobileDistortionPass.Substrate
#else
	#define SceneTexturesStruct DistortionPass.SceneTextures
	#define DistortionParams DistortionPass.DistortionParams
	#define SubstrateStruct DistortionPass.Substrate
#endif

#include "SceneTexturesCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "DistortionCommon.ush"

#ifndef ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define ADAPTIVE_VOLUMETRIC_SHADOW_MAP 0
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define AVSM DistortionPass.AVSM
#include "HeterogeneousVolumes/HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

// SUBSTRATE_TODO enable distortion on mobile.
#if SUBSTRATE_ENABLED && SHADING_PATH_MOBILE
#undef SUBSTRATE_ENABLED
#define SUBSTRATE_ENABLED 0
#endif

#if SUBSTRATE_ENABLED
#include "Substrate/SubstrateEvaluation.ush"
#endif

void ClipOccludedPixel(float PixelZ, float SceneZ)
{
	float ClipDepth = PixelZ - SceneZ;
	clip(ClipDepth);
}

/** output distortion offsets as color so they can be accumulated (via blending) */
void Main(
	FVertexFactoryInterpolantsVSToPS Interpolants,
	float4	PixelPosition	: TEXCOORD6,
	in float4 SvPosition	: SV_Position,
	in FStereoPSInput StereoInput
	OPTIONAL_IsFrontFace,
	out float4 OutColor		: SV_Target0
#if SUBSTRATE_ENABLED
	,
	out float2 OutVarianceCoverage : SV_Target1
	,
	out float  OutClosestDepthMeter : SV_Target2
#endif
	)
{
	StereoSetupPS(StereoInput);
	const uint EyeIndex = GetEyeIndex(StereoInput);

#if SUBSTRATE_ENABLED
	OutVarianceCoverage = float2(0.0f, 0.0f);
	OutClosestDepthMeter = MaxHalfFloat;
#endif

	// material parameter inputs
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);

#if SHADING_PATH_MOBILE && MATERIAL_SHOULD_DISABLE_DEPTH_TEST==0
	// Mobile specific: 
	// distortion primtitives rendered without depth testing
	// discard pixels that are hidden by manually comparing them to a scene depth
	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	ClipOccludedPixel(SvPosition.z, LookupDeviceZ(BufferUV));
#endif
	
	// material distortion offset
	half3 Normal = GetMaterialNormal(MaterialParameters, PixelMaterialInputs);

	// Prevent silhouettes from geometry that is in front of distortion from being seen in the distortion 
	float2 NDC = (MaterialParameters.ScreenPosition.xy / MaterialParameters.ScreenPosition.w);
	float2 ScreenUV = NDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;

	FMaterialRefractionData RefractionData = GetMaterialRefraction(PixelMaterialInputs);


#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

	// Retrieve rough refraction variance
	float Sigma = 0.0f;
	float3 RefractionWorldNormal = MaterialParameters.CameraVector;

#if SUBSTRATE_OPTIMIZED_UNLIT

	// Unlit BSDF goes through the SubstrateTree to support weighting operations. Technically, layering and mixing could also be supported.
	float3	UnlitSurfaceLuminancePostCoverage = 0.0f;
	float	UnlitSurfaceCoverage = 0.0f;
	float3	UnlitSurfaceTransmittancePreCoverage = 0.0f;
	SubstratePixelHeader.SubstrateUpdateTreeUnlit(
		uint2(SvPosition.xy),
		MaterialParameters.CameraVector,
		SubstrateData,
		UnlitSurfaceLuminancePostCoverage,
		UnlitSurfaceCoverage,
		UnlitSurfaceTransmittancePreCoverage,
		RefractionWorldNormal);

	const float MaterialIOR = GetMaterialRefractionIOR(RefractionData); // Unlit does not have any surface F0 so we only use the root node input.

	const bool TryToClip = true; // In this case rough refractions are not supported, we can thus always clip.

#if DISTORTION_ACCOUNT_FOR_COVERAGE
	const float RefractionCoverage = UnlitSurfaceCoverage;	// Coverage of the material.
#else
	const float RefractionCoverage = 1.0;					// Behavior where coverage is always ignored (legacy support)
#endif

	const float RefractionLobeVariance = 0.0f;
	OutVarianceCoverage = float2(RefractionLobeVariance, RefractionCoverage);

#else

	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;

		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, true/*bRoughnessTracking*/);
		SubstratePixelHeader.SubstrateUpdateTree(V, Settings);

		RefractionWorldNormal = SubstratePixelHeader.SubstrateTree.Operators[SubstrateData.OperatorIndex].TopDownRefractionWorldNormal;

		FSubstrateLobeStatistic RefractionLobeStat = SubstratePixelHeader.SubstrateTree.Operators[SubstrateData.OperatorIndex].TopDownRefractionLobeStat;

		Sigma = RefractionLobeStat.Sigma;
	}

	// We accumulate F0 only from the top layer to evaluate the material F0 used to evaluate distortion
	float3 TopF0 = 0.0f;
	Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
	{
	#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]
		const bool bIsVisible = SubstrateIsBSDFVisible(BSDF);
		if (bIsVisible)
		{
			TopF0 += BSDF.TopLayerDataWeight * SubstrateGetBSDFSpecularF0(BSDF);
		}
	#undef BSDF
	}
	TopF0 = SanitizeF0(TopF0);	// Sanitize, in case a Add node has been used

	const float MaterialIOR = DielectricF0RGBToIor(TopF0);

	const FSubstrateOperator RootOperator	= SubstratePixelHeader.SubstrateTree.Operators[SubstrateData.OperatorIndex];
#if DISTORTION_ACCOUNT_FOR_COVERAGE
	const float RefractionCoverage		= RootOperator.Coverage;														// Coverage of the material
#else
	const float RefractionCoverage		= 1.0;																			// Behavior where coverage is always ignored (legacy support)
#endif
	const float RefractionLobeVariance	= Sigma;																		// The lobe variance that can be added to in the en recover a single lobe.

	// Write out variance (as this is more linear to accumulate than roughness).
	OutVarianceCoverage = float2(RefractionLobeVariance, RefractionCoverage);

	// Only allow clipping if roughness is close to 0.
	const bool TryToClip = false;  //OutVarianceCoverage.x < 0.01f; // SUBSTRATE_TODO || RefractionCoverage <= 0.0f;	// Have an option to account for coverage or not?, ignored by default

#endif

	const float RefractionDepthMeter = length(MaterialParameters.WorldPosition_CamRelative) * CENTIMETER_TO_METER;	// Distance for camera in meters
	OutClosestDepthMeter = RefractionDepthMeter;

	// Compute UV distortion
	float2 BufferUVDistortion = ComputeBufferUVDistortion(
		MaterialParameters, PixelMaterialInputs, ResolvedView,
		RefractionWorldNormal, MaterialIOR,
		DistortionParams, ScreenUV, RefractionData, TryToClip, EyeIndex);

	// Sample depth at distortion offset
	float2 DistortBufferUV = ScreenUV + BufferUVDistortion;

#else // SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	float  Specular = GetMaterialSpecular(PixelMaterialInputs);
	float3 F0 = ComputeF0(Specular, BaseColor, Metallic);
	float MaterialIOR = DielectricF0RGBToIor(F0);

	// Compute UV distortion
	float2 BufferUVDistortion = ComputeBufferUVDistortion(
		MaterialParameters, PixelMaterialInputs, ResolvedView,
		MaterialParameters.WorldNormal, MaterialIOR,
		DistortionParams, ScreenUV, RefractionData, true, EyeIndex);

	// Sample depth at distortion offset
	float2 DistortBufferUV = ScreenUV + BufferUVDistortion;

#if DISTORTION_ACCOUNT_FOR_COVERAGE
	const float RefractionCoverage = GetMaterialOpacity(PixelMaterialInputs);	// Coverage of the material.
#else
	const float RefractionCoverage = 1.0f;										// Behavior where coverage is always ignored (legacy support)
#endif
	const float RefractionLobeVariance = 0.0f;
#if SUBSTRATE_ENABLED
	OutVarianceCoverage = float2(RefractionLobeVariance, RefractionCoverage);
#endif

#endif // SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	float DistortSceneDepth = CalcSceneDepth(DistortBufferUV);

	// Post process UV distortion according to depth
	PostProcessUVDistortion(MaterialParameters, PixelMaterialInputs, DistortSceneDepth, BufferUVDistortion, RefractionData);

	// store positive and negative offsets separately
	float2 PosOffset = max(BufferUVDistortion,0);
	float2 NegOffset = abs(min(BufferUVDistortion,0));

	// output positives in R|G channels and negatives in B|A channels
	OutColor = float4(PosOffset.x,PosOffset.y,NegOffset.x,NegOffset.y);

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
	if (DistortionPass.UseAVSM)
	{
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
		float Transmittance = saturate(AVSM_SampleTransmittance(TranslatedWorldPosition, 0));
		float ApplyDistortion = (Transmittance >= DistortionPass.TransmittanceThreshold) ? 1.0 : 0.0;
		OutColor *= ApplyDistortion;
	}
#endif
}


