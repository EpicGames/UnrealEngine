// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionAccumulatePixelShader.usf: Vertex/Hull/Domain shader for accumulating distortion offsets
=============================================================================*/

#include "Common.ush"

// Reroute distortion pass uniform buffer.
#if SHADING_PATH_MOBILE
	#define MobileSceneTextures MobileDistortionPass.SceneTextures
	#define DistortionParams MobileDistortionPass.DistortionParams
#else
	#define SceneTexturesStruct DistortionPass.SceneTextures
	#define DistortionParams DistortionPass.DistortionParams
#endif

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FDistortAccumulateVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4	PixelPosition	: TEXCOORD6;
	float4	Position		: SV_POSITION;
	FStereoVSOutput StereoOutput;
};

#define VS_OUTPUT_TYPE FDistortAccumulateVSToPS

#if VERTEXSHADER

/** transform mesh as normal */
void Main(
	FVertexFactoryInput Input,
	out VS_OUTPUT_TYPE Output
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, Output.StereoOutput);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4	WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	// Isolate instructions used for world position offset on xbox 360, 
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	{
		WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
	}
	
	Output.Position = mul(WorldPosition, ResolvedView.TranslatedWorldToClip);
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif

	Output.PixelPosition = WorldPosition;
}

#endif // VERTEXSHADER
