// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================
	DistortApplyScreenPixelShader.usf: Pixel shader for rendering screen distortion pass
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"

#include "/Engine/Private/Substrate/Substrate.ush"

#if USE_ROUGH_REFRACTION && SUBSTRATE_ENABLED
#include "/Engine/Private/Substrate/SubstrateStatisticalOperators.ush"
#include "BlueNoise.ush"
#endif


static const half InvDistortionScaleBias = 1 / 4.0f;

/** 
 * contains accumulated distortion values as 
 * R=positive horizontal offset
 * G=positive vertical offset
 * B=negative horizontal offset
 * A=negative vertical offset 
 */

Texture2D		SceneColorTexture;
SamplerState	SceneColorTextureSampler;
Texture2D		SceneDepthMeterTexture;
SamplerState	SceneDepthMeterTextureSampler;
  
 #if USE_MSAA
Texture2DMS<float4> DistortionMSAATexture;
Texture2DMS<float4> SceneColorMSAATexture;
Texture2DMS<float4> SceneDepthMeterMSAATexture;
Texture2DMS<float4> RefractionVarianceCoverageMSAATexture;
Texture2DMS<float4> RefractionClosestDepthMeterMSAATexture;
#if COMPOSITE_STANDARD_TRANSLUCENT
Texture2DMS<float4> StandardTranslucentColorAlphaMSAATexture;
Texture2DMS<float4> StandardTranslucentTransmittanceMSAATexture;
#endif
#else
Texture2D			DistortionTexture;
SamplerState		DistortionTextureSampler;
Texture2D<float4>	RefractionVarianceCoverageTexture;
SamplerState		RefractionVarianceCoverageSampler;
Texture2D<float4>	RefractionClosestDepthMeterTexture;
SamplerState		RefractionClosestDepthMeterSampler;
#if COMPOSITE_STANDARD_TRANSLUCENT
Texture2D			StandardTranslucentColorAlphaTexture;
SamplerState		StandardTranslucentColorAlphaTextureSampler;
Texture2D			StandardTranslucentTransmittanceTexture;
SamplerState		StandardTranslucentTransmittanceTextureSampler;
#endif
#endif

float RefractionMaxMipLevel;
float StandardDevInPixelForMip0;
float BlurDitheringScaleFactor;

// TexCoords [0,1] where 0 is the left edge and 1 is the right edge.
// This means that in ISR they go over the whole doublewide.
int2 TexToPixCoords(float2 TexCoord)
{
	float2 SampleCoords = (TexCoord.xy * View.BufferSizeAndInvSize.xy);
	return int2(SampleCoords);
}

/** distorts screen texture using accumulated distortion offsets */
void Main(
	in noperspective float4 TexCoord: TEXCOORD0,
	in FStereoPSInput StereoInput,
	out float4 OutColor : SV_Target0
#if USE_MSAA
	, in uint SampleIndex : SV_SampleIndex
#endif
	)
{
	// sample accumulated distortion and apply inverse scale
	const int2 PixCoord = TexToPixCoords(TexCoord.xy);
#if USE_MSAA
	half4 AccumDist = DistortionMSAATexture.Load(PixCoord, SampleIndex);
#else
	half4 AccumDist = Texture2DSample(DistortionTexture,DistortionTextureSampler,TexCoord.xy);
#endif

	// apply inverse scale
	// offset = [R-B,G-A] 
	half2 DistBufferUVOffset = (AccumDist.rg - AccumDist.ba);

	//Scale by the screen size and a fudge factor to come close to the offset values we would have had under normal circumstances before my changes. Also flip Y and invert the precision bias scale.
	DistBufferUVOffset *= InvDistortionScaleBias;
	
#if USE_ROUGH_REFRACTION && SUBSTRATE_ENABLED

#if USE_MSAA
	half2 RefractionVarianceCoverage= RefractionVarianceCoverageMSAATexture.Load(PixCoord, SampleIndex).xy;
	half  RefractionDepthMeter		= RefractionClosestDepthMeterMSAATexture.Load(PixCoord, SampleIndex).x;
#else
	half2 RefractionVarianceCoverage= Texture2DSample(RefractionVarianceCoverageTexture, RefractionVarianceCoverageSampler, TexCoord.xy).xy;
	half  RefractionDepthMeter		= Texture2DSample(RefractionClosestDepthMeterTexture, RefractionClosestDepthMeterSampler, TexCoord.xy).x;
#endif

	const float2 SVPos				=  TexCoord.xy * View.BufferSizeAndInvSize.xy + View.ViewRectMin.xy;
	const float2 NoDistortionCoord	= (TexCoord.xy * View.BufferSizeAndInvSize.xy - View.ViewRectMin.xy);
	const float2 NoDistortionUVs	= NoDistortionCoord * View.ViewSizeAndInvSize.zw;

	float2 NewBufferUV = NoDistortionUVs + DistBufferUVOffset;
	FLATTEN if (any(NewBufferUV < 0.0f) || any(NewBufferUV > 1.0f))
	{
		NewBufferUV = NoDistortionUVs;
	}

	half SceneDepthMeter			= 0.0f;
#if USE_MSAA
	SceneDepthMeter = SceneDepthMeterMSAATexture.Load(int2(NewBufferUV * View.ViewSizeAndInvSize.xy), SampleIndex).x;
#else
	const half SceneDepthMeterMip0 = Texture2DSampleLevel(SceneDepthMeterTexture, SceneDepthMeterTextureSampler, NewBufferUV, 0).x;
	const half SceneDepthMeterMipN = Texture2DSampleLevel(SceneDepthMeterTexture, SceneDepthMeterTextureSampler, NewBufferUV, 4).x;
	SceneDepthMeter = min(SceneDepthMeterMip0, SceneDepthMeterMipN);
#endif

	// We evaluate the gaussian blur needed to match the refraction of a surface at a distance behind a surface exhibiting rough refraction.
	// For that we used a process and blurred depth buffer storing linear depth in meter. This allows to have soft transitions around edges.
	// It is not perfect however with large depth difference.
	// The gaussian blur standard deviation is estiamted from the pixel footprint of the projected radiance at a certain depth from a rough surfaces using GetVarianceFromRoughness and the Thales theorem.
	// The correct blured scene mip level is then evaluated to make sure the standard devianttion in pixel matches the one of the projected rough refraction gaussian kernel.

	half FrontSurfaceRefractionDepthCm			= max(0.0f, SceneDepthMeter - RefractionDepthMeter) * METER_TO_CENTIMETER;
	half RefractionRoughness					= saturate(SubstrateLobeVarianceToRoughness(RefractionVarianceCoverage.x));
	const float StandardDeviationCm				= sqrt(GetVarianceFromRoughness(RefractionRoughness));
	const float StandardDeviationCmForThickness = StandardDeviationCm * FrontSurfaceRefractionDepthCm;
	const float VarianceAtDepth					= StandardDeviationCmForThickness * StandardDeviationCmForThickness;

	half RefractionMipLevel = 0.0;
	{
		const float2 PixelRadiusCm2 = METER_TO_CENTIMETER * SceneDepthMeter * GetTanHalfFieldOfView() * View.ViewSizeAndInvSize.zw;
		const float PixelRadiusCm = min(PixelRadiusCm2.x, PixelRadiusCm2.y);

		const float PixelStandardDevInPixel = StandardDeviationCmForThickness / PixelRadiusCm;

		// Jitter UV according to MIP level to hide low res texels. (also tried mip index dithering: good but less effective)
		// It does add a bit of extra blurring, but practically acceptable and it is not that much more.
		if(BlurDitheringScaleFactor > 0.0f) // make this an option
		{
			NewBufferUV += (BlueNoiseVec2(SVPos.xy, View.StateFrameIndexMod8) - 0.5f) * BlurDitheringScaleFactor * PixelStandardDevInPixel * View.ViewSizeAndInvSize.zw;
		}

		// The standard deviation in pixel for mip0 = StandardDevInPixelForMip0.
		// It means that for each mips N, with the same gaussian with same variance is applied, the standard deviation in pixel of image at source resolution is = StandardDevInPixelForMip0 * 2^M.
		// We now want to compute the mip to select for the current pixel PixelStandardDevInPixel.
		// So PixelStandardDevInPixel = StandardDevInPixelForMip0 * 2^M
		//	  PixelStandardDevInPixel / StandardDevInPixelForMip0 = 2^M
		// and because y = z ^ x => x = log(y) / log(z)
		// we obtain  M = log(PixelStandardDevInPixel / StandardDevInPixelForMip0) / log(2)

		// Compute mip level according to VarianceAtDepth at a depth of FrontSurfaceRefractionDepthCm
		RefractionMipLevel = log(PixelStandardDevInPixel / StandardDevInPixelForMip0) / log(2);
	}
	RefractionMipLevel = min(RefractionMipLevel, RefractionMaxMipLevel);

#if USE_MSAA
	if (RefractionMipLevel < 0.001f)
	{
		// Sample the full res MSAA source texture
		OutColor = SceneColorMSAATexture.Load(int2(NoDistortionCoord), SampleIndex);
	}
	else
	{
		OutColor = Texture2DSampleLevel(SceneColorTexture, SceneColorTextureSampler, NewBufferUV, RefractionMipLevel);
	}
#else
	OutColor = Texture2DSampleLevel(SceneColorTexture, SceneColorTextureSampler, NewBufferUV, RefractionMipLevel);
#endif

	// Reveal the original scene color unrefracted/sharp as a function of coverage.
	const float Coverage = RefractionVarianceCoverage.y;
	if (Coverage < 1.0f)
	{
		float4 OriginalSceneColor = 0.0f;
	#if USE_MSAA
		OriginalSceneColor = SceneColorMSAATexture.Load(int2(NoDistortionUVs * View.BufferSizeAndInvSize.xy), SampleIndex);
	#else
		OriginalSceneColor = Texture2DSample(SceneColorTexture, SceneColorTextureSampler, NoDistortionUVs);
	#endif

		OutColor = lerp(OriginalSceneColor, OutColor, Coverage);
	}

#else // USE_ROUGH_REFRACTION

	float2 NewBufferUV = TexCoord.xy + DistBufferUVOffset;

	// If we're about to sample outside the valid BufferUVMinMax, set to 0 distortion.
	FLATTEN if ( NewBufferUV.x < View.BufferBilinearUVMinMax.x || NewBufferUV.x > View.BufferBilinearUVMinMax.z ||
				 NewBufferUV.y < View.BufferBilinearUVMinMax.y || NewBufferUV.y > View.BufferBilinearUVMinMax.w )
	{
		NewBufferUV = TexCoord.xy;
	}

	// sample screen using offset coords
#if USE_MSAA
	OutColor = SceneColorMSAATexture.Load(TexToPixCoords(NewBufferUV), SampleIndex);
#else
	OutColor = Texture2DSample(SceneColorTexture, SceneColorTextureSampler, NewBufferUV);
#endif

#endif // USE_ROUGH_REFRACTION
}

void Merge(
	in noperspective float4 TexCoord: TEXCOORD0,
	in FStereoPSInput StereoInput,
	out float4 OutColor : SV_Target0
#if USE_MSAA
	, in uint SampleIndex : SV_SampleIndex
#endif
	)
{
#if USE_MSAA
	int2 PixCoord = TexToPixCoords(TexCoord.xy);
	OutColor = SceneColorMSAATexture.Load(PixCoord, SampleIndex);
#if COMPOSITE_STANDARD_TRANSLUCENT
	const float4 TranslucentColorAlpha		= StandardTranslucentColorAlphaMSAATexture.Load(PixCoord, SampleIndex);
	const float3 TranslucentTransmittance	= StandardTranslucentTransmittanceMSAATexture.Load(PixCoord, SampleIndex).rgb;
#endif

#else // USE_MSAA

	OutColor = Texture2DSample(SceneColorTexture, SceneColorTextureSampler, TexCoord.xy);
#if COMPOSITE_STANDARD_TRANSLUCENT
	const float4 TranslucentColorAlpha		= Texture2DSample(StandardTranslucentColorAlphaTexture, StandardTranslucentColorAlphaTextureSampler, TexCoord.xy);
	const float3 TranslucentTransmittance	= Texture2DSample(StandardTranslucentTransmittanceTexture, StandardTranslucentTransmittanceTextureSampler, TexCoord.xy).rgb;
#endif

#endif // USE_MSAA

#if COMPOSITE_STANDARD_TRANSLUCENT
	float GreyScaleModulateColorBackgroundVisibility = dot(TranslucentTransmittance, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));

	OutColor = float4(
		OutColor.rgb * TranslucentColorAlpha.a * TranslucentTransmittance + TranslucentColorAlpha.rgb,
		OutColor.a   * TranslucentColorAlpha.a * GreyScaleModulateColorBackgroundVisibility);
#endif
}

Texture2D		DistortionAccumulateTexture;
SamplerState	DistortionAccumulateSampler;

void Merge_Mobile(
	in noperspective float4 TexCoord: TEXCOORD0,
	out HALF4_TYPE OutColor : SV_Target0
)
{
	half4 AccumDist = DistortionAccumulateTexture.Sample(DistortionAccumulateSampler, TexCoord.xy);
	// apply inverse scale
	// offset = [R-B,G-A] 
	half2 DistBufferUVOffset = (AccumDist.rg - AccumDist.ba);
	//Scale by the screen size and a fudge factor to come close to the offset values we would have had under normal circumstances before my changes. Also flip Y and invert the precision bias scale.
	DistBufferUVOffset *= InvDistortionScaleBias;
	float2 NewBufferUV = TexCoord.xy + DistBufferUVOffset;
	// If we're about to sample outside the valid BufferUVMinMax, set to 0 distortion.
	FLATTEN if ( NewBufferUV.x < View.BufferBilinearUVMinMax.x || NewBufferUV.x > View.BufferBilinearUVMinMax.z ||
				 NewBufferUV.y < View.BufferBilinearUVMinMax.y || NewBufferUV.y > View.BufferBilinearUVMinMax.w )
	{
		NewBufferUV = TexCoord.xy;
	}

	OutColor = SceneColorTexture.Sample(SceneColorTextureSampler, NewBufferUV.xy);
}

