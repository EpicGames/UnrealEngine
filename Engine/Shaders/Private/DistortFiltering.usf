// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortFiltering.usf: Pixel shader for filtering the scene color in preparation for rough refraction
=============================================================================*/

#include "Common.ush"



#ifdef COPYSCENECOLORDEPTHTEXTUREPS

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D SceneDepthTexture;
SamplerState SceneDepthSampler;

float MaxExposedLuminance;

void CopySceneColorDepthTexturePS(FScreenVertexOutput Input, out float4 OutColor : SV_Target0, out float OutDepthMeter : SV_Target1)
{
	float2 SVPos = Input.UV * View.ViewSizeAndInvSize.xy + View.ViewRectMin.xy;

	float2 TextureUV = SVPos * View.BufferSizeAndInvSize.zw;
	float4 LinearColor = Texture2DSample(SceneColorTexture, SceneColorSampler, TextureUV);

	// Clamping to a certain max luminance help to reduce bright specular highlight flickering, even with temporal AA stabilization.
	LinearColor.rgb = min(LinearColor.rgb, MaxExposedLuminance.xxx);

	float DeviceZ = Texture2DSample(SceneDepthTexture, SceneDepthSampler, TextureUV).r;
	DeviceZ = max(0.000000000001, DeviceZ);	// TODO: avoid bad values when DeviceZ is far=0 when using inverted z
	float4 SVPosition = float4(SVPos, DeviceZ, 1);
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SVPosition);

	OutColor = float4(LinearColor.rgb, 1.0f);
	OutDepthMeter = length(TranslatedWorldPosition) * CENTIMETER_TO_METER;
}

#endif // COPYSCENECOLORPS



#ifdef DOWNSAMPLECOLORCS

int2 SrcMipResolution;
int2 DstMipResolution;
SamplerState SourceSampler;
Texture2D<float4> SourceTexture;
RWTexture2D<float4> OutTextureMipColor;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void DownsampleColorCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy >= uint2(DstMipResolution)))
	{
		return;
	}

	float2 ScaledUVs = (float2(ThreadId.xy) * 2.0 + 1.0f) / float2(SrcMipResolution);
	float4 OutColor = SourceTexture.SampleLevel(SourceSampler, ScaledUVs, 0); // No need to specify mip index, the SRV is created for the acual mip level.

	OutTextureMipColor[uint2(ThreadId.xy)] = OutColor;
}

#endif // DOWNSAMPLECOLORCS



#ifdef FILTERCOLORCS

uint SampleCount;
int2 SrcMipResolution;
int2 DstMipResolution;
float4 BlurDirection;
SamplerState SourceSampler;
Texture2D<float4> SourceTexture;
RWTexture2D<float4> OutTextureMipColor;

#define MAX_FILTER_SAMPLE_COUNT_RADIUS	32
#define MAX_FILTER_SAMPLE_COUNT	(MAX_FILTER_SAMPLE_COUNT_RADIUS*2+1)
float4 SampleOffsetsWeights[MAX_FILTER_SAMPLE_COUNT];

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void FilterColorCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy >= uint2(DstMipResolution)))
	{
		return;
	}
	float2 ScaledUVs = (float2(ThreadId.xy) * 2.0 + 1.0f) / float2(SrcMipResolution);

	float4 OutColor = 0;
	
	for (uint i = 0; i < SampleCount; ++i)
	{
		float2 SampleOffsetWeight = SampleOffsetsWeights[i].xy;

		float4 Sample = SourceTexture.SampleLevel(SourceSampler, ScaledUVs + SampleOffsetWeight.xx * BlurDirection.xy, 0); // No need to specify mip index, the SRV is created for the acual mip level.

		OutColor += Sample * SampleOffsetWeight.y;
	}

	OutTextureMipColor[uint2(ThreadId.xy)] = OutColor;
}

#endif // FILTERCOLORCS


