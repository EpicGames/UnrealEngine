// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionCommon.ush
=============================================================================*/

#include "Common.ush"

float2 ComputeBufferUVDistortion(
	in FMaterialPixelParameters MaterialParameters, in FPixelMaterialInputs PixelMaterialInputs, in ViewState ResolvedView,
	in half3 WorldNormal, in float InMaterialIOR,
	in float4 DistortionParameters, in float2 ScreenUV, in FMaterialRefractionData RefractionData, in bool TryToClip, in uint EyeIndex)
{
	half3 ViewNormal = normalize(TransformWorldVectorToView(WorldNormal));

#if REFRACTION_USE_PIXEL_NORMAL_OFFSET
	half3 ViewVertexNormal = TransformTangentVectorToView(MaterialParameters, float3(0, 0, 1));
	// Treat refraction of 1 as no refraction, to be consistent with IOR mode
	float2 ViewportUVDistortion = (ViewVertexNormal.xy - ViewNormal.xy) * (GetMaterialRefractionPixelNormalStrength(RefractionData) - 1);

#elif REFRACTION_USE_INDEX_OF_REFRACTION
	// we assume the camera is in air
	float AirIOR = 1.0f;
#if SUBSTRATE_OPTIMIZED_UNLIT
	float MaterialIOR = GetUnlitMaterialRefractionIOR(RefractionData);
#else
	float MaterialIOR = GetMaterialRefractionIOR(RefractionData);
#endif
	float2 ViewportUVDistortion = ViewNormal.xy * (MaterialIOR - AirIOR);

#elif REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0
	// we assume the camera is in air
	float AirIOR = 1.0f;
#if SUBSTRATE_OPTIMIZED_UNLIT
	float MaterialIOR = GetUnlitMaterialRefractionIOR(RefractionData);
#else
	float MaterialIOR = InMaterialIOR;
#endif
	float2 ViewportUVDistortion = ViewNormal.xy * (MaterialIOR - AirIOR);

#elif REFRACTION_USE_2D_OFFSET
	float2 ViewportUVDistortion = GetMaterialRefraction2DOffset(RefractionData);

#elif REFRACTION_USE_NONE
	float2 ViewportUVDistortion = 0.0f;

#else
#error Unkown refraction mode.
#endif

	float2 BufferUVDistortion = ViewportUVDistortion * ResolvedView.ViewSizeAndInvSize.xy * ResolvedView.BufferSizeAndInvSize.zw;

	// Clip if the distortion distance (squared) is too small to be noticed.
	// This will result in a less expensive apply pass since the clipped pixels won't set stencil to pass.
	// We only clip if the distortion is less than the size of a pixel of the unscaled viewport, i.e. dynamic resolution = 100%, to ensure a good reconstruction.
	const float2 FullResolutionDistortionPixelSize = ResolvedView.BufferSizeAndInvSize.zw;
	if (TryToClip && 
		all(abs(BufferUVDistortion) < FullResolutionDistortionPixelSize))
	{
		clip(-1.0f);
	}

	// When ISR is enabled we store two FOVs in the DistortionParameters and compute the aspect ratio here
#if INSTANCED_STEREO
	float InvTanHalfFov = DistortionParameters[EyeIndex];
	float Ratio = DistortionParameters.z / DistortionParameters.w;
#else
	float InvTanHalfFov = DistortionParameters.x;
	float Ratio = DistortionParameters.y;
#endif

	// InvTanHalfFov only apply a correction for the distortion to be the same in screen space space whatever the FoV is (to make it consistent accross player setup).
	// However without taking depth into account, the distortion will actually be stronger the further away the camera is from the distortion surface.
	// So when zoomed-in the distortion will be higher than expected.
	// To fix this, a scale of 100/SurfaceDepth would be a good approximation to make the distortion properly scaled when the surface is zoomed in and/or further away (with distortion authored at 1meter being the reference strength)

	// Fix for Fov and aspect.
	float2 FovFix = float2(InvTanHalfFov, Ratio*InvTanHalfFov);
	//A fudge factor scale to bring values close to what they would have been under usual circumstances prior to this change.
	const float OffsetFudgeFactor = 0.00023;
	BufferUVDistortion *= DistortionParameters.zw * float2(OffsetFudgeFactor, -OffsetFudgeFactor) * FovFix;

	return BufferUVDistortion;
}


void PostProcessUVDistortion(
	in FMaterialPixelParameters MaterialParameters, in FPixelMaterialInputs PixelMaterialInputs, 
	in float DistortSceneDepth,	inout float2 BufferUVDistortion, in FMaterialRefractionData RefractionData)
{
	// Soft thresholding 
	float Bias = -RefractionData.RefractionDepthBias;
	float Range = clamp(abs(Bias * 0.5f), 0, 50);
	float Z = DistortSceneDepth;
	float ZCompare = MaterialParameters.ScreenPosition.w;
	float InvWidth = 1.0f / max(1.0f, Range);
	BufferUVDistortion *= saturate((Z - ZCompare) * InvWidth + Bias);

	//Scale up for better precision in low/subtle refractions at the expense of artefacts at higher refraction.
	static const half DistortionScaleBias = 4.0f;
	BufferUVDistortion *= DistortionScaleBias;
}



