// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================================
	DitheredTransitionStencil.usf: Fills the stencil buffer with the current dithered transition mask.
=============================================================================================================*/

#include "Common.ush"

float DitheredTransitionFactor;

// .xy = Viewport offset
//  .z = Stencil masked value
//  .w = Stencil cleared value
uint4 StencilOffsetAndValues;

RWTexture2D<uint> StencilOutput;

half GetDitherValue(float2 InPosition)
{
	float RandCos = cos(dot(floor(InPosition), float2(347.83451793f,3343.28371963f)));
	float RandomVal = frac(RandCos * 1000.f);
	half RetVal = (DitheredTransitionFactor < 0.f) ?
		(DitheredTransitionFactor + 1.f > RandomVal) :
		(DitheredTransitionFactor < RandomVal);
	return (RetVal - 0.001f);
}

void Main(float4 SvPosition : SV_POSITION)
{
	if (abs(DitheredTransitionFactor) > 0.001f)
	{
		clip(GetDitherValue(SvPosition.xy));
	}
}

[numthreads(8, 8, 1)]
void MainCS(uint3 PixelPos : SV_DispatchThreadID)
{
	// This should assumes no pre-existing values in the stencil buffer, so any pixels
	// that pass the transition factor test will set the sandbox bit, otherwise the
	// default clear value should be exported (skipping the normal stencil target clear).
	uint StencilValue = (StencilOffsetAndValues.w & 0xFF);

	if (abs(DitheredTransitionFactor) > 0.001f)
	{
		if (GetDitherValue(float2(PixelPos.xy)) >= 0.0f)
		{
			// The clip() instruction kills any pixels < 0, so we flip the test here
			// and change from default clear to masked value in the case where the test
			// passes.
			StencilValue = (StencilOffsetAndValues.z & 0xFF);
		}
	}

	StencilOutput[StencilOffsetAndValues.xy + PixelPos.xy] = StencilValue;
}