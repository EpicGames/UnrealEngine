// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// A high-precision floating point type, consisting of two 32-bit floats (High & Low).
// 'High' stores the core value, 'Low' stores the residual error that couldn't fit in 'High'.
// This combination has 2x23=46 significant bits, providing twice the precision a float offers.
// Operations are slower than floats, but faster than doubles on consumer GPUs (with potentially greater support)
// Platforms that don't support fused multiply-add and INVARIANT may see decreased performance or reduced precision.
// 
// Based on:
//  [0] Thall, A. (2006), Extended-precision floating-point numbers for GPU computation.
//  [1] Mioara Maria Joldes, Jean-Michel Muller, Valentina Popescu. (2017), Tight and rigourous error bounds for basic building blocks of double-word arithmetic.
//  [2] Vincent Lefevre, Nicolas Louvet, Jean-Michel Muller, Joris Picot, and Laurence Rideau. (2022), Accurate Calculation of Euclidean Norms using Double-Word Arithmetic
//  [3] Jean-Michel Muller and Laurence Rideau. (2022), Formalization of Double-Word Arithmetic, and Comments on "Tight and Rigorous Error Bounds for Basic Building Blocks of Double-Word Arithmetic"
//  [4] T. J. Dekker. (1971), A floating-point technique for extending the available precision.

#ifndef UE_DF_FORCE_FP32_OPS
#define UE_DF_FORCE_FP32_OPS 0
#endif

#ifndef UE_DF_NO_FAST_MATH
#define UE_DF_NO_FAST_MATH 0
#endif

#include "RandomPCG.ush"

/* FDFScalar */

#define FDFType FDFScalar
#define FDFTypeDeriv FDFScalarDeriv
#define FFloatType float
#define FBoolType bool
#define GNumComponents 1
#define DFConstructor MakeDFScalar
#if COMPILER_SUPPORTS_HLSL2021
struct FDFType;
FDFType DFAdd(FDFType Lhs, FDFType Rhs);
FDFType DFAdd(FDFType Lhs, FFloatType Rhs);
FDFType DFSubtract(FDFType Lhs, FDFType Rhs);
FDFType DFSubtract(FDFType Lhs, FFloatType Rhs);
FDFType DFMultiply(FDFType Lhs, FDFType Rhs);
FDFType DFMultiply(FDFType Lhs, FFloatType Rhs);
FDFType DFDivide(FDFType Lhs, FDFType Rhs);
FDFType DFDivide(FDFType Lhs, FFloatType Rhs);
FDFType DFNegate(FDFType V);
FBoolType DFGreater(FDFType Lhs, FFloatType Rhs);
FBoolType DFLess(FDFType Lhs, FFloatType Rhs);
FFloatType DFDemote(FDFType V);
#endif

struct FDFScalar
{
	float High;
	float Low;

#if COMPILER_SUPPORTS_HLSL2021
	/*FDFType operator+(FDFType Rhs) { return DFAdd(this, Rhs); }
	FDFType operator+(FFloatType Rhs) { return DFAdd(this, Rhs); }
	FDFType operator-(FDFType Rhs) { return DFSubtract(this, Rhs); }
	FDFType operator-(FFloatType Rhs) { return DFSubtract(this, Rhs); }
	FDFType operator*(FDFType Rhs) { return DFMultiply(this, Rhs); }
	FDFType operator*(FFloatType Rhs) { return DFMultiply(this, Rhs); }
	FDFType operator/(FDFType Rhs) { return DFDivide(this, Rhs); }
	FDFType operator/(FFloatType Rhs) { return DFDivide(this, Rhs); }*/
	//Commented out until supported by HlslParser
	/*FDFType operator-() { return DFNegate(this); }
	FBoolType operator>(FFloatType Rhs) { return DFGreater(this, Rhs); }
	FBoolType operator<(FFloatType Rhs) { return DFLess(this, Rhs); }*/

	FFloatType Demote() { return DFDemote(this); }
#endif
};

#include "DoubleFloatOperations.ush"
#undef FDFType
#undef FDFTypeDeriv
#undef FFloatType
#undef FBoolType
#undef GNumComponents
#undef DFConstructor

FDFScalar MakeDFVector(FDFScalar X) { return X; }

FDFScalar DFGetComponent(float V, int C) { return MakeDFScalar(V, 0); }
FDFScalar DFGetComponent(FDFScalar V, int C) { return V; }

float DFLength(float V) { return abs(V); }
float DFNormalize(float V) { return (float)sign(V); }
FDFScalar DFLength(FDFScalar V) { return DFAbs(V); }
FDFScalar DFNormalize(FDFScalar V) { return DFPromote(V); }
float DFNormalizeDemote(FDFScalar V) { return DFSign(V); }

/* FDFVector2 */

#define FDFType FDFVector2
#define FDFTypeDeriv FDFVector2Deriv
#define FFloatType float2
#define FBoolType bool2
#define GNumComponents 2
#define DFConstructor MakeDFVector2
#define DFBroadcastFunc DFBroadcast2
#include "DoubleFloatVectorDefinition.ush"
#undef FDFType
#undef FDFTypeDeriv
#undef FFloatType
#undef FBoolType
#undef GNumComponents
#undef DFConstructor
#undef DFBroadcastFunc

FDFVector2 MakeDFVector(FDFScalar X, FDFScalar Y) { return MakeDFVector2(float2(X.High, Y.High), float2(X.Low, Y.Low)); }

/* FDFVector3 */

#define FDFType FDFVector3
#define FDFTypeDeriv FDFVector3Deriv
#define FFloatType float3
#define FBoolType bool3
#define GNumComponents 3
#define DFConstructor MakeDFVector3
#define DFBroadcastFunc DFBroadcast3
#include "DoubleFloatVectorDefinition.ush"
#undef FDFType
#undef FDFTypeDeriv
#undef FFloatType
#undef FBoolType
#undef GNumComponents
#undef DFConstructor
#undef DFBroadcastFunc

FDFVector3 MakeDFVector(FDFScalar   X, FDFScalar   Y, FDFScalar Z) { return MakeDFVector3(float3( X.High,  Y.High, Z.High), float3( X.Low,  Y.Low, Z.Low)); }
FDFVector3 MakeDFVector(FDFScalar   X, FDFVector2 YZ)              { return MakeDFVector3(float3( X.High, YZ.High),         float3( X.Low, YZ.Low)); }
FDFVector3 MakeDFVector(FDFVector2 XY, FDFScalar   Z)              { return MakeDFVector3(float3(XY.High,  Z.High),         float3(XY.Low,  Z.Low)); }

/* FDFVector4 */

#define FDFType FDFVector4
#define FDFTypeDeriv FDFVector4Deriv
#define FFloatType float4
#define FBoolType bool4
#define GNumComponents 4
#define DFConstructor MakeDFVector4
#define DFBroadcastFunc DFBroadcast4
#include "DoubleFloatVectorDefinition.ush"
#undef FDFType
#undef FDFTypeDeriv
#undef FFloatType
#undef FBoolType
#undef GNumComponents
#undef DFConstructor
#undef DFBroadcastFunc

FDFVector4 MakeDFVector(FDFScalar    X, FDFScalar    Y, FDFScalar   Z, FDFScalar W) { return MakeDFVector4(float4(  X.High,   Y.High,  Z.High, W.High), float4(  X.Low,   Y.Low,  Z.Low, W.Low)); }
FDFVector4 MakeDFVector(FDFScalar    X, FDFScalar    Y, FDFVector2 ZW)              { return MakeDFVector4(float4(  X.High,   Y.High, ZW.High),         float4(  X.Low,   Y.Low, ZW.Low)); }
FDFVector4 MakeDFVector(FDFScalar    X, FDFVector2  YZ, FDFScalar   W)              { return MakeDFVector4(float4(  X.High,  YZ.High,  W.High),         float4(  X.Low,  YZ.Low,  W.Low)); }
FDFVector4 MakeDFVector(FDFVector2  XY, FDFScalar    Z, FDFScalar   W)              { return MakeDFVector4(float4( XY.High,   Z.High,  W.High),         float4( XY.Low,   Z.Low,  W.Low)); }
FDFVector4 MakeDFVector(FDFVector2  XY, FDFVector2  ZW)                             { return MakeDFVector4(float4( XY.High,  ZW.High),                  float4( XY.Low,  ZW.Low)); }
FDFVector4 MakeDFVector(FDFScalar    X, FDFVector3 YZW)                             { return MakeDFVector4(float4(  X.High, YZW.High),                  float4(  X.Low, YZW.Low)); }
FDFVector4 MakeDFVector(FDFVector3 XYZ, FDFScalar    W)                             { return MakeDFVector4(float4(XYZ.High,   W.High),                  float4(XYZ.Low,   W.Low)); }

FDFVector4 MakeDFVector(FDFVector3 In, float W) { return MakeDFVector4(float4(In.High, W), float4(In.Low, 0)); }


#define DFGetX(V) DFGetComponent(V, 0)
#define DFGetY(V) DFGetComponent(V, 1)
#define DFGetZ(V) DFGetComponent(V, 2)
#define DFGetW(V) DFGetComponent(V, 3)

FDFVector3 DFGetXYZ(FDFVector4 V)
{
	return MakeDFVector3(V.High.xyz, V.Low.xyz);
}

// Put the components of V at the specified indices in a new DF vector

FDFScalar DFSwizzle(FDFScalar V, int C0)  { return V; }
FDFScalar DFSwizzle(FDFVector2 V, int C0) { return DFGetComponent(V, C0); }
FDFScalar DFSwizzle(FDFVector3 V, int C0) { return DFGetComponent(V, C0); }
FDFScalar DFSwizzle(FDFVector4 V, int C0) { return DFGetComponent(V, C0); }

FDFVector2 DFSwizzle(FDFScalar V, int C0, int C1)  { return MakeDFVector(V, V); }
FDFVector2 DFSwizzle(FDFVector2 V, int C0, int C1) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1)); }
FDFVector2 DFSwizzle(FDFVector3 V, int C0, int C1) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1)); }
FDFVector2 DFSwizzle(FDFVector4 V, int C0, int C1) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1)); }

FDFVector3 DFSwizzle(FDFScalar V, int C0, int C1, int C2)  { return MakeDFVector(V, V, V); }
FDFVector3 DFSwizzle(FDFVector2 V, int C0, int C1, int C2) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2)); }
FDFVector3 DFSwizzle(FDFVector3 V, int C0, int C1, int C2) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2)); }
FDFVector3 DFSwizzle(FDFVector4 V, int C0, int C1, int C2) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2)); }

FDFVector4 DFSwizzle(FDFScalar V, int C0, int C1, int C2, int C3)  { return MakeDFVector(V, V, V, V); }
FDFVector4 DFSwizzle(FDFVector2 V, int C0, int C1, int C2, int C3) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2), DFGetComponent(V, C3)); }
FDFVector4 DFSwizzle(FDFVector3 V, int C0, int C1, int C2, int C3) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2), DFGetComponent(V, C3)); }
FDFVector4 DFSwizzle(FDFVector4 V, int C0, int C1, int C2, int C3) { return MakeDFVector(DFGetComponent(V, C0), DFGetComponent(V, C1), DFGetComponent(V, C2), DFGetComponent(V, C3)); }

// Needed for new translator
FDFScalar DFDistance(FDFScalar A, FDFScalar B) { return DFAbs(DFSubtract(A, B)); }
FDFScalar DFDistance(FDFVector2 A, FDFVector2 B) { return DFLength(DFSubtract(A, B)); }
FDFScalar DFDistance(FDFVector3 A, FDFVector3 B) { return DFLength(DFSubtract(A, B)); }
FDFScalar DFDistance(FDFVector4 A, FDFVector4 B) { return DFLength(DFSubtract(A, B)); }

#define DFHackToFloat DFDemote

// Transforms *to* absolute world space
struct FDFMatrix
{
	float4x4 M;
	float3 PostTranslation; // Added to result, *after* multiplying 'M'
};

float4x4 Make4x3Matrix(float4x4 M)
{
	// Explicitly ignore the 4th column of the input
	float4x4 Result;
	Result[0] = float4(M[0].xyz, 0.0f);
	Result[1] = float4(M[1].xyz, 0.0f);
	Result[2] = float4(M[2].xyz, 0.0f);
	Result[3] = float4(M[3].xyz, 1.0f);
	return Result;
}

float4x4 MakeTranslationMatrix(float3 Offset)
{
	float4x4 Result;
	Result[0] = float4(1.0f, 0.0f, 0.0f, 0.0f);
	Result[1] = float4(0.0f, 1.0f, 0.0f, 0.0f);
	Result[2] = float4(0.0f, 0.0f, 1.0f, 0.0f);
	Result[3] = float4(Offset, 1.0f);
	return Result;
}

FDFMatrix MakeDFMatrix4x3(float3 PostTranslation, float4x4 InMatrix)
{
	FDFMatrix Result;
	Result.PostTranslation = PostTranslation;
	Result.M = Make4x3Matrix(InMatrix);
	return Result;
}

FDFMatrix MakeDFMatrix(float3 PostTranslation, float4x4 InMatrix)
{
	FDFMatrix Result;
	Result.PostTranslation = PostTranslation;
	Result.M = InMatrix;
	return Result;
}

FDFVector3 DFGetOrigin(FDFMatrix InMatrix)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(InMatrix.PostTranslation + InMatrix.M[3].xyz);
#elif UE_DF_NO_FAST_MATH
	return DFTwoSum(InMatrix.PostTranslation, InMatrix.M[3].xyz);
#else
	return DFFastTwoSum(InMatrix.PostTranslation, InMatrix.M[3].xyz);
#endif
}

// Transforms *from* absolute world space
struct FDFInverseMatrix
{
	float4x4 M;
	float3 PreTranslation; // Subtracted from input position *before* multiplying 'M'
#if !COMPILER_SUPPORTS_HLSL2021
	int Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

FDFInverseMatrix MakeDFInverseMatrix(float3 PreTranslation, float4x4 InMatrix)
{
	FDFInverseMatrix Result;
	Result.PreTranslation = PreTranslation;
	Result.M = InMatrix;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FDFInverseMatrix MakeDFInverseMatrix4x3(float3 PreTranslation, float4x4 InMatrix)
{
	FDFInverseMatrix Result;
	Result.PreTranslation = PreTranslation;
	Result.M = Make4x3Matrix(InMatrix);
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FDFVector3 DFMultiply(float3 Vector, FDFMatrix InMatrix)
{
	float4 Result = mul(float4(Vector, 1), InMatrix.M);
	Result /= Result.w;
	FDFVector3 V = DFTwoSum(InMatrix.PostTranslation, Result.xyz);
	return V;
}

// Use this if the matrix is an affine transform, e.g. translation, rotation, scaling, shearing or combinations of these.
// If your matrix is non-affine, like projection, use DFMultiply instead.
FDFVector3 DFMultiplyAffine(float3 Vector, FDFMatrix InMatrix)
{
	float4 Result = mul(float4(Vector, 1), InMatrix.M);
	FDFVector3 V = DFTwoSum(InMatrix.PostTranslation, Result.xyz);
	return V;
}

FDFVector4 DFMultiply(float4 Vector, FDFMatrix InMatrix)
{
	float4 Result = mul(Vector, InMatrix.M);
	FDFVector4 V = DFTwoSum(float4(InMatrix.PostTranslation, 0), Result);
	return V;
}

float3 DFMultiplyDemote(FDFVector3 Vector, FDFInverseMatrix InMatrix)
{
	float3 V = DFFastSubtractDemote(Vector, InMatrix.PreTranslation);
	float4 Result = mul(float4(V, 1), InMatrix.M);
	Result /= Result.w;
	return Result.xyz;
}

float3 DFFastMultiplyDemote(FDFVector3 Vector, FDFInverseMatrix InMatrix)
{
	float3 V = DFFastLocalSubtractDemote(Vector, InMatrix.PreTranslation);
	float4 Result = mul(float4(V, 1), InMatrix.M);
	Result /= Result.w;
	return Result.xyz;
}

// Use this if the matrix is an affine transform, e.g. translation, rotation, scaling, shearing or combinations of these.
// If your matrix is non-affine, like (de-)projection, use DFMultiplyDemote instead.
float3 DFMultiplyAffineDemote(FDFVector3 Vector, FDFInverseMatrix InMatrix)
{
	float3 V = DFFastSubtractDemote(Vector, InMatrix.PreTranslation);
	float4 Result = mul(float4(V, 1), InMatrix.M);
	return Result.xyz;
}

// Use this if the matrix is an affine transform, e.g. translation, rotation, scaling, shearing or combinations of these.
// If your matrix is non-affine, like (de-)projection, use DFFastMultiplyDemote instead.
float3 DFFastMultiplyAffineDemote(FDFVector3 Vector, FDFInverseMatrix InMatrix)
{
	float3 V = DFFastLocalSubtractDemote(Vector, InMatrix.PreTranslation);
	float4 Result = mul(float4(V, 1), InMatrix.M);
	return Result.xyz;
}

float4 DFMultiplyDemote(FDFVector4 Vector, FDFInverseMatrix InMatrix)
{
	float4 V = DFFastSubtractDemote(Vector, float4(InMatrix.PreTranslation, 0));
	return mul(V, InMatrix.M);
}

float4 DFFastMultiplyDemote(FDFVector4 Vector, FDFInverseMatrix InMatrix)
{
	float4 V = DFFastLocalSubtractDemote(Vector, float4(InMatrix.PreTranslation, 0));
	return mul(V, InMatrix.M);
}

FDFInverseMatrix DFMultiply(FDFInverseMatrix Lhs, float4x4 Rhs)
{
	float4x4 M = mul(Lhs.M, Rhs);
	float3 PreTranslation = Lhs.PreTranslation;
	return MakeDFInverseMatrix(PreTranslation, M);
}
FDFMatrix DFMultiply(float4x4 Lhs, FDFMatrix Rhs)
{
	float4x4 M = mul(Lhs, Rhs.M);
	float3 PostTranslation = Rhs.PostTranslation;
	return MakeDFMatrix(PostTranslation, M);
}

// Requires M to be an affine transform matrix, e.g. translation, rotation, scaling, shearing or combinations of these.
// Do not use with projection matrices
float4x4 MultiplyAffineTranslation(float4x4 M, float3 Translation)
{
	float4x4 Result = M;
	// M * translate(PostTranslation)
	Result[3].xyz += Translation;
	return Result;
}

float4x4 MultiplyTranslation(float4x4 M, float3 Translation)
{
	return mul(M, MakeTranslationMatrix(Translation));
}

float4x4 MultiplyTranslation(float3 Translation, float4x4 M)
{
	return mul(MakeTranslationMatrix(Translation), M);
}

// Perform Lhs * T, where T is a translation matrix with the specified vector
float4x4 DFMultiplyTranslationDemote(FDFMatrix Lhs, FDFVector3 Rhs)
{
	float3 Translation = DFFastAddDemote(Lhs.PostTranslation, Rhs);
	float4x4 Result = MultiplyTranslation(Lhs.M, Translation);
	return Result;
}
// Perform Lhs * T, where T is a translation matrix with the specified vector
// and Lhs is an affine transform matrix (no projection). 
float4x4 DFMultiplyAffineTranslationDemote(FDFMatrix Lhs, FDFVector3 Rhs)
{
	float3 Translation = DFFastAddDemote(Lhs.PostTranslation, Rhs);
	float4x4 Result = MultiplyAffineTranslation(Lhs.M, Translation);
	return Result;
}
// Same as DFMultiplyTranslationDemote, but assumes the translation vector
// roughly cancels out the translation in the matrix.
// More formally, Lhs.PostTranslation and -Rhs should meet the requirements of DFFastLocalSubtractDemote
float4x4 DFFastMultiplyTranslationDemote(FDFMatrix Lhs, FDFVector3 Rhs)
{
#if UE_DF_NO_FAST_MATH
	return DFMultiplyTranslationDemote(Lhs, Rhs);
#else
	float3 Translation = DFFastLocalSubtractDemote(Lhs.PostTranslation, DFNegate(Rhs));
	float4x4 Result = MultiplyTranslation(Lhs.M, Translation);
	return Result;
#endif
}
// Same as DFMultiplyAffineTranslationDemote, but assumes the translation vector
// roughly cancels out the translation in the matrix.
// More formally, Lhs.PostTranslation and -Rhs should meet the requirements of DFFastLocalSubtractDemote
float4x4 DFFastMultiplyAffineTranslationDemote(FDFMatrix Lhs, FDFVector3 Rhs)
{
#if UE_DF_NO_FAST_MATH
	return DFMultiplyAffineTranslationDemote(Lhs, Rhs);
#else
	float3 Translation = DFFastLocalSubtractDemote(Lhs.PostTranslation, DFNegate(Rhs));
	float4x4 Result = MultiplyAffineTranslation(Lhs.M, Translation);
	return Result;
#endif
}

float4x4 DFMultiplyTranslationDemote(FDFMatrix Lhs, float3 Rhs)
{
	float3 Translation = INVARIANT_ADD(Lhs.PostTranslation, Rhs);
	float4x4 Result = MultiplyTranslation(Lhs.M, Translation);
	return Result;
}
float4x4 DFMultiplyAffineTranslationDemote(FDFMatrix Lhs, float3 Rhs)
{
	float3 Translation = INVARIANT_ADD(Lhs.PostTranslation, Rhs);
	float4x4 Result = MultiplyAffineTranslation(Lhs.M, Translation);
	return Result;
}
// Perform Lhs * T, where T is a translation matrix with the specified vector
FDFMatrix DFMultiplyTranslation(FDFMatrix Lhs, float3 Rhs)
{
	FDFVector3 Translation = DFTwoSum(Lhs.PostTranslation, Rhs);

	float3 PostTranslation = Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Lhs.M;
#else
	float4x4 M = MultiplyTranslation(Lhs.M, Translation.Low);
#endif
	return MakeDFMatrix(PostTranslation, M);
}
// Perform Lhs * T, where T is a translation matrix with the specified vector
// and Lhs is an affine transform matrix (no projection). 
FDFMatrix DFMultiplyAffineTranslation(FDFMatrix Lhs, float3 Rhs)
{
	FDFVector3 Translation = DFTwoSum(Lhs.PostTranslation, Rhs);

	float3 PostTranslation = Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Lhs.M;
#else
	float4x4 M = MultiplyAffineTranslation(Lhs.M, Translation.Low);
#endif
	return MakeDFMatrix(PostTranslation, M);
}
FDFMatrix DFMultiplyTranslation(float4x4 Lhs, FDFVector3 Rhs)
{
	FDFVector3 Translation = Rhs;

	float3 PostTranslation = Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Lhs;
#else
	float4x4 M = MultiplyTranslation(Lhs, Translation.Low);
#endif
	return MakeDFMatrix(PostTranslation, M);
}
FDFMatrix DFMultiplyAffineTranslation(float4x4 Lhs, FDFVector3 Rhs)
{
	FDFVector3 Translation = Rhs;

	float3 PostTranslation = Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Lhs;
#else
	float4x4 M = MultiplyAffineTranslation(Lhs, Translation.Low);
#endif
	return MakeDFMatrix(PostTranslation, M);
}

// Perform Lhs * T, where T is a translation matrix with the specified vector
FDFInverseMatrix DFMultiplyTranslation(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
	FDFVector3 Translation = DFFastAdd(Lhs, -Rhs.PreTranslation); 

	float3 PreTranslation = -Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Rhs.M;
#else
	float4x4 M = MultiplyTranslation(Translation.Low, Rhs.M);
#endif
	return MakeDFInverseMatrix(PreTranslation, M);
}
FDFInverseMatrix DFMultiplyTranslationAffine(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
	return DFMultiplyTranslation(Lhs, Rhs); // no speed-up for affine MultiplyTranslation(float3, float4x4)
}

float4x4 DFMultiplyTranslationDemote(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
	float3 Translation = DFFastAddDemote(Lhs, -Rhs.PreTranslation);
	float4x4 Result = MultiplyTranslation(Translation, Rhs.M);
	return Result;
}
float4x4 DFMultiplyTranslationAffineDemote(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
	return DFMultiplyTranslationDemote(Lhs, Rhs); // no speed-up for affine MultiplyTranslation(float3, float4x4)
}

// Same as DFMultiplyTranslationDemote, but assumes the translation vector
// roughly cancels out the translation in the matrix.
// More formally, Lhs and Rhs.PreTranslation should meet the requirements of DFFastLocalSubtractDemote
float4x4 DFFastMultiplyTranslationDemote(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
#if UE_DF_NO_FAST_MATH
	return DFMultiplyTranslationDemote(Lhs, Rhs);
#else
	float3 Translation = DFFastLocalSubtractDemote(Lhs, Rhs.PreTranslation);
	float4x4 Result = MultiplyTranslation(Translation, Rhs.M);
	return Result;
#endif
}
float4x4 DFFastMultiplyTranslationAffineDemote(FDFVector3 Lhs, FDFInverseMatrix Rhs)
{
	return DFFastMultiplyTranslationDemote(Lhs, Rhs); // no speed-up for affine MultiplyTranslation(float3, float4x4)
}

FDFInverseMatrix DFMultiplyTranslation(float3 Lhs, FDFInverseMatrix Rhs)
{
	FDFVector3 Translation = DFTwoSum(Lhs, -Rhs.PreTranslation);

	float3 PreTranslation = -Translation.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Rhs.M;
#else
	float4x4 M = MultiplyTranslation(Translation.Low, Rhs.M);
#endif
	return MakeDFInverseMatrix(PreTranslation, M);
}
FDFInverseMatrix DFMultiplyTranslationAffine(float3 Lhs, FDFInverseMatrix Rhs)
{
	return DFMultiplyTranslation(Lhs, Rhs); // no speed-up for affine MultiplyTranslation(float3, float4x4)
}

FDFInverseMatrix DFMultiplyTranslation(FDFVector3 Lhs, float4x4 Rhs)
{
	float3 PreTranslation = -Lhs.High;
#if UE_DF_FORCE_FP32_OPS
	float4x4 M = Rhs;
#else
	float4x4 M = MultiplyTranslation(Lhs.Low, Rhs);
#endif
	return MakeDFInverseMatrix(PreTranslation, M);
}
FDFInverseMatrix DFMultiplyTranslationAffine(FDFVector3 Lhs, float4x4 Rhs)
{
	return DFMultiplyTranslation(Lhs, Rhs); // no speed-up for affine MultiplyTranslation(float3, float4x4)
}

float3x3 DFToFloat3x3(FDFMatrix Value)
{
	return (float3x3)Value.M;
}

float3x3 DFToFloat3x3(FDFInverseMatrix Value)
{
	return (float3x3)Value.M;
}

float3 DFMultiplyVector(float3 Vector, FDFMatrix InMatrix)
{
	return mul(Vector, (float3x3)InMatrix.M);
}

float3 DFMultiplyVector(float3 Vector, FDFInverseMatrix InMatrix)
{
	return mul(Vector, (float3x3)InMatrix.M);
}

FDFMatrix DFPromote(FDFMatrix Value) { return Value; }
FDFInverseMatrix DFPromoteInverse(FDFInverseMatrix Value) { return Value; }
FDFMatrix  DFPromote(float4x4 Value) { return MakeDFMatrix((float3)0, Value); }
FDFInverseMatrix DFPromoteInverse(float4x4 Value) { return MakeDFInverseMatrix((float3)0, Value); }

float4x4 DFDemote(FDFMatrix V)
{
	return MultiplyTranslation(V.M, V.PostTranslation);
}

// Requires M to be an affine transform matrix, e.g. translation, rotation, scaling, shearing or combinations of these.
// Do not use with projection matrices
float4x4 DFDemoteAffine(FDFMatrix V)
{
	return MultiplyAffineTranslation(V.M, V.PostTranslation);
}

float4x4 DFDemote(float4x4 V)
{
	return V;
}

float4x4 DFDemote(FDFInverseMatrix V)
{
	return MultiplyTranslation(-V.PreTranslation, V.M);
}

float4 DFTransformLocalToTranslatedWorld(float3 LocalPosition, FDFMatrix LocalToWorld, FDFVector3 PreViewTranslation)
{
	// Multiply local position with LocalToWorld 3x3 first for high precision rotation and scale, then move the world position part to translated world before adding it  
	float3 RotatedScaledPosition = FMA(LocalPosition.xxx, LocalToWorld.M[0].xyz, FMA(LocalPosition.yyy, LocalToWorld.M[1].xyz, LocalPosition.zzz * LocalToWorld.M[2].xyz));

#if UE_DF_NO_FAST_MATH
	float3 TranslatedWorldPositionOrigin = DFAddDemote(DFGetOrigin(LocalToWorld), PreViewTranslation);
#else
	float3 TranslatedWorldPositionOrigin = DFFastLocalSubtractDemote(LocalToWorld.PostTranslation, DFNegate(PreViewTranslation)) + LocalToWorld.M[3].xyz;
#endif
	
	return float4(RotatedScaledPosition + TranslatedWorldPositionOrigin, 1.0f);
}

// Translate the input vector with PreViewTranslation. Assumes the result fits in FP32 (i.e. is close to the view origin)
float3 DFFastToTranslatedWorld(FDFVector3 WorldPosition, FDFVector3 PreViewTranslation)
{
	return DFFastLocalSubtractDemote(WorldPosition, DFNegate(PreViewTranslation));
}

float3 DFFastToTranslatedWorld(float3 WorldPosition, FDFVector3 PreViewTranslation)
{
	return DFFastLocalSubtractDemote(WorldPosition, DFNegate(PreViewTranslation));
}

// Transforms a WorldToX matrix to a TranslatedWorldToX matrix.
float4x4 DFFastToTranslatedWorld(FDFInverseMatrix WorldToLocal, FDFVector3 PreViewTranslation)
{
	return DFFastMultiplyTranslationAffineDemote(DFNegate(PreViewTranslation), WorldToLocal);
}

// Transforms a XToWorld matrix to a XToTranslatedWorld matrix.
float4x4 DFFastToTranslatedWorld(FDFMatrix LocalToWorld, FDFVector3 PreViewTranslation)
{
	return DFFastMultiplyAffineTranslationDemote(LocalToWorld, PreViewTranslation);
}

FDFVector3 DFMultiplyLHSAndFastTwoSum(float3 Lhs, float Factor, float3 Rhs)
{
	const float3 S = INVARIANT_FMA(Lhs, Factor, Rhs);
	const float3 E = INVARIANT_SUB(Rhs, INVARIANT_FMA(-Lhs, Factor, S));
	return MakeDFVector3(S, E);
}

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
FDFMatrix WaveReadLaneAt(FDFMatrix In, uint SrcIndex)
{
	FDFMatrix Result;
	Result.M[0] = WaveReadLaneAt(In.M[0], SrcIndex);
	Result.M[1] = WaveReadLaneAt(In.M[1], SrcIndex);
	Result.M[2] = WaveReadLaneAt(In.M[2], SrcIndex);
	Result.M[3] = WaveReadLaneAt(In.M[3], SrcIndex);
	Result.PostTranslation = WaveReadLaneAt(In.PostTranslation, SrcIndex);
	return Result;
}

FDFInverseMatrix WaveReadLaneAt(FDFInverseMatrix In, uint SrcIndex)
{
	FDFInverseMatrix Result;
	Result.M[0] = WaveReadLaneAt(In.M[0], SrcIndex);
	Result.M[1] = WaveReadLaneAt(In.M[1], SrcIndex);
	Result.M[2] = WaveReadLaneAt(In.M[2], SrcIndex);
	Result.M[3] = WaveReadLaneAt(In.M[3], SrcIndex);
	Result.PreTranslation = WaveReadLaneAt(In.PreTranslation, SrcIndex);
	return Result;
}
#endif

uint4 DFHashPCG(FDFVector2 In)
{
	int4 V1 = asint(float4(In.High.x, In.High.y, In.Low.x, In.Low.y));
	return Rand4DPCG32(V1);
}

uint DFHash_LCG(FDFVector2 In)
{
	const uint M = 1664525u;
	const uint A = 1013904223u;
	uint V1 = asuint(In.High.x) + asuint(In.High.y)
			+ asuint(In.Low.x)  + asuint(In.Low.y);
	uint V2 = V1 * M + A;
	// second round improves visual quality significantly, even though the compiler premultiplies these values, so no extra ops
	uint V3 = V2 * M + A;
	return V3;
}

// High-quality hash, but more expensive.
// If you need a statistically robust hash, choose this.
// For noise that looks visually plausible, choose LCG.
uint3 DFHashPCG(FDFVector3 In)
{
	int3 V1 = asint(In.High);
	int3 V2 = asint(In.Low);

	uint3 H1 = Rand3DPCG32(V1);
	uint3 H2 = Rand3DPCG32(H1 + V2);

	return H2;
}

// Low-quality hash, but instruction count is roughly 1/8th of PCG approach.
uint DFHash_LCG(FDFVector3 In)
{
	const uint M = 1664525u;
	const uint A = 1013904223u;
	uint V1 = asuint(In.High.x) + asuint(In.High.y) + asuint(In.High.z) 
			+ asuint(In.Low.x)  + asuint(In.Low.y)  + asuint(In.Low.z);
	uint V2 = V1 * M + A;
	// second round improves visual quality significantly, even though the compiler premultiplies these values, so no extra ops
	uint V3 = V2 * M + A;
	return V3;
}

uint4 DFHash_PCG(FDFVector4 In)
{
	int4 V1 = asint(In.High);
	int4 V2 = asint(In.Low);

	uint4 H1 = Rand4DPCG32(V1);
	uint4 H2 = Rand4DPCG32(H1 + V2);

	return H2;
}

uint DFHash_LCG(FDFVector4 In)
{
	const uint M = 1664525u;
	const uint A = 1013904223u;
	uint V1 = asuint(In.High.x) + asuint(In.High.y) + asuint(In.High.z) + asuint(In.High.w)
			+ asuint(In.Low.x)  + asuint(In.Low.y)  + asuint(In.Low.z)  + asuint(In.Low.w);
	uint V2 = V1 * M + A;
	// second round improves visual quality significantly, even though the compiler premultiplies these values, so no extra ops
	uint V3 = V2 * M + A;
	return V3;
}