// Copyright Epic Games, Inc. All Rights Reserved.

// This constructs a FDFScalar, but does not 'rebalance' the input floats.
// Use DFTwoSum instead if the input is not already in the correct high/low format.
FDFType DFConstructor(FFloatType High, FFloatType Low)
{
	FDFType Result;
	Result.High = High;
	Result.Low = Low;
	return Result;
}

FDFType DFPromote(FDFType Value) { return Value; }

// Convert to double float
FDFType DFPromote(FFloatType Value) { return DFConstructor(Value, (FFloatType)0); }

// Truncate to single precision float
// (This could just return Value.High if you're sure the input is in the correct DF format)
FFloatType DFDemote(FDFType Value) { return Value.High + Value.Low; }

FFloatType DFDemote(FFloatType Value) { return Value; }

FDFType DFNegate(FDFType Value)
{
	return DFConstructor(-Value.High, -Value.Low);
}

FFloatType DFSign(FDFType Value)
{
	return (FFloatType)sign(Value.High);
}

FDFType DFAbs(FDFType Value)
{
	const FFloatType Sign = DFSign(Value);
	return DFConstructor(Sign * Value.High, Sign * Value.Low);
}

/**************/
/** ADDITION **/
/**************/

// Sum up two single precision vectors to one double precision vector.
// [1] Algorithm 2
FDFType DFTwoSum(FFloatType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs + Rhs);
#else
	const FFloatType S = INVARIANT_ADD(Lhs, Rhs);
	const FFloatType V = INVARIANT_SUB(S, Lhs);
	const FFloatType Q = INVARIANT_SUB(S, V);
	const FFloatType R = INVARIANT_SUB(Lhs, Q);
	const FFloatType T = INVARIANT_SUB(Rhs, V);
	const FFloatType Y = INVARIANT_ADD(R, T);
	return DFConstructor(S, Y);
#endif
}

// An optimized version of DFTwoSum, under the assumption that
// a = 0 or b = 0, or e1 >= e2, where e1 and e2 are the exponents of a and b respectively.
// [1] Algorithm 1
// The worst case precision if this assumption is violated, is just regular fp32 precision
FDFType DFFastTwoSum(FFloatType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs + Rhs);
#else
	const FFloatType S = INVARIANT_ADD(Lhs, Rhs);
	const FFloatType T = INVARIANT_SUB(S, Lhs);
	const FFloatType E = INVARIANT_SUB(Rhs, T);
	return DFConstructor(S, E);
#endif
}

// Alias for DFTwoSum
FDFType DFAdd(FFloatType Lhs, FFloatType Rhs) { return DFTwoSum(Lhs, Rhs); }
FDFType DFFastAdd(FFloatType Lhs, FFloatType Rhs) { return DFTwoSum(Lhs, Rhs); }

// [1] Algorithm 6
FDFType DFAdd(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High + Rhs.High);
#else
	// lossless add
	FDFType S = DFTwoSum(Lhs.High, Rhs.High);
	const FDFType T = DFTwoSum(Lhs.Low, Rhs.Low);
	// merge and rebalance
	S.Low += T.High;
	S = DFFastTwoSum(S.High, S.Low);
	S.Low += T.Low;
	S = DFFastTwoSum(S.High, S.Low);
	return S;
#endif
}

// [1] Algorithm 4
FDFType DFAdd(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High + Rhs);
#else
	const FDFType S = DFTwoSum(Lhs.High, Rhs);
	const FFloatType T = Lhs.Low + S.Low;
	return DFFastTwoSum(S.High, T);
#endif
}

FDFType DFAdd(FFloatType Lhs, FDFType Rhs) { return DFAdd(Rhs, Lhs); }

FFloatType DFAddDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High + Rhs.High;
#else
	// lossless add
	FDFType S = DFTwoSum(Lhs.High, Rhs.High);
	const FDFType T = DFTwoSum(Lhs.Low, Rhs.Low);
	// merge and rebalance
	S.Low += T.High;
	S = DFFastTwoSum(S.High, S.Low);
	S.Low += T.Low;
	return S.High + S.Low;
#endif
}

FFloatType DFAddDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High + Rhs;
#else
	const FDFType S = DFTwoSum(Lhs.High, Rhs);
	const FFloatType T = Lhs.Low + S.Low;
	return S.High + T;
#endif
}

FFloatType DFAddDemote(FFloatType Lhs, FDFType Rhs) { return DFAddDemote(Rhs, Lhs); }
FFloatType DFAddDemote(FFloatType Lhs, FFloatType Rhs) { return Lhs + Rhs; }

FDFType DFFastAdd(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High + Rhs.High);
#else
	FDFType S = DFTwoSum(Lhs.High, Rhs.High);
	S.Low += Lhs.Low + Rhs.Low;
	S = DFFastTwoSum(S.High, S.Low);
	return S;
#endif
}

FDFType DFFastAdd(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High + Rhs);
#else
	FDFType S = DFTwoSum(Lhs.High, Rhs);
	S.Low += Lhs.Low;
	S = DFFastTwoSum(S.High, S.Low);
	return S;
#endif
}

FDFType DFFastAdd(FFloatType Lhs, FDFType Rhs) { return DFFastAdd(Rhs, Lhs); }

FFloatType DFFastAddDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High + Rhs.High;
#else
	FDFType S = DFTwoSum(Lhs.High, Rhs.High);
	S.Low += Lhs.Low + Rhs.Low;
	return S.High + S.Low;
#endif
}

FFloatType DFFastAddDemote(FDFType Lhs, FFloatType Rhs) { return DFAddDemote(Lhs, Rhs); }
FFloatType DFFastAddDemote(FFloatType Lhs, FDFType Rhs) { return DFFastAddDemote(Rhs, Lhs); }
FFloatType DFFastAddDemote(FFloatType Lhs, FFloatType Rhs) { return Rhs + Lhs; }

/*****************/
/** SUBTRACTION **/
/*****************/

FDFType DFSubtract(FFloatType Lhs, FFloatType Rhs) { return DFAdd(Lhs, -Rhs); }
FDFType DFSubtract(FDFType Lhs, FDFType Rhs) { return DFAdd(Lhs, DFNegate(Rhs)); }
FDFType DFSubtract(FDFType Lhs, FFloatType Rhs) { return DFAdd(Lhs, -Rhs); }
FDFType DFSubtract(FFloatType Lhs, FDFType Rhs) { return DFAdd(Lhs, DFNegate(Rhs)); }

FFloatType DFSubtractDemote(FFloatType Lhs, FFloatType Rhs) { return Lhs - Rhs; }
FFloatType DFSubtractDemote(FDFType Lhs, FDFType Rhs) { return DFAddDemote(Lhs, DFNegate(Rhs)); }
FFloatType DFSubtractDemote(FDFType Lhs, FFloatType Rhs) { return DFAddDemote(Lhs, -Rhs); }
FFloatType DFSubtractDemote(FFloatType Lhs, FDFType Rhs) { return DFAddDemote(Lhs, DFNegate(Rhs)); }

FDFType DFFastSubtract(FFloatType Lhs, FFloatType Rhs) { return DFFastAdd(Lhs, -Rhs); }
FDFType DFFastSubtract(FDFType Lhs, FDFType Rhs) { return DFFastAdd(Lhs, DFNegate(Rhs)); }
FDFType DFFastSubtract(FDFType Lhs, FFloatType Rhs) { return DFFastAdd(Lhs, -Rhs); }
FDFType DFFastSubtract(FFloatType Lhs, FDFType Rhs) { return DFFastAdd(Lhs, DFNegate(Rhs)); }

FFloatType DFFastSubtractDemote(FFloatType Lhs, FFloatType Rhs) { return Lhs - Rhs; }
FFloatType DFFastSubtractDemote(FDFType Lhs, FDFType Rhs) { return DFFastAddDemote(Lhs, DFNegate(Rhs)); }
FFloatType DFFastSubtractDemote(FDFType Lhs, FFloatType Rhs) { return DFFastAddDemote(Lhs, -Rhs); }
FFloatType DFFastSubtractDemote(FFloatType Lhs, FDFType Rhs) { return DFFastAddDemote(Lhs, DFNegate(Rhs)); }

// Subtract using only 2 ops instead of ~20, and convert the result to fp32 (1 op)
// CAREFUL, this has strict input preconditions to get output that has any precision at all.
// Assuming you require a precision of 2^-4, the precondition for Lhs and Rhs is
//   (abs(Lhs) < 2^20 && abs(Rhs) < 2^20) ||
//   ( abs(Lhs) < 2^43 && abs(Rhs) < 2^43 && Sign(Lhs) == Sign(Rhs) && abs(trunc(log2(abs(Lhs))) - trunc(log2(abs(Rhs)))) < 1 )
// As the preconditions imply, this does not work for addition due to FP rounding.
FFloatType DFFastLocalSubtractDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High - Rhs.High;
#elif UE_DF_NO_FAST_MATH
	return DFSubtractDemote(Lhs, Rhs);
#else
	const FFloatType High = INVARIANT_SUB(Lhs.High, Rhs.High);
	const FFloatType Low = INVARIANT_SUB(Lhs.Low, Rhs.Low);
	const FFloatType Sum = INVARIANT_ADD(High, Low);
	return Sum;
#endif
}

FFloatType DFFastLocalSubtractDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High - Rhs;
#elif UE_DF_NO_FAST_MATH
	return DFSubtractDemote(Lhs, Rhs);
#else
	const FFloatType High = INVARIANT_SUB(Lhs.High, Rhs);
	const FFloatType Sum = INVARIANT_ADD(High, Lhs.Low);
	return Sum;
#endif
}

FFloatType DFFastLocalSubtractDemote(FFloatType Lhs, FDFType Rhs)
{
	return DFFastLocalSubtractDemote(DFPromote(Lhs), Rhs);
}

/********************/
/** MULTIPLICATION **/
/********************/

// [4] Algorithm 'mul12' (Page 241)
FDFType DFTwoProductNoFMA(FFloatType Lhs, FFloatType Rhs)
{
	const FFloatType Constant = 4097; // = 2^(t - t/2) + 1, where t is the number of mantissa bits
	const FFloatType Px = INVARIANT_MUL(Lhs, Constant);
	const FFloatType Hx = INVARIANT_ADD(INVARIANT_SUB(Lhs, Px), Px);
	const FFloatType Tx = INVARIANT_SUB(Lhs, Hx);
	const FFloatType Py = INVARIANT_MUL(Rhs, Constant);
	const FFloatType Hy = INVARIANT_ADD(INVARIANT_SUB(Rhs, Py), Py);
	const FFloatType Ty = INVARIANT_SUB(Rhs, Hy);
	const FFloatType P = INVARIANT_MUL(Hx, Hy);
	const FFloatType Q = INVARIANT_ADD(INVARIANT_MUL(Hx, Ty), INVARIANT_MUL(Tx, Hy));
	const FFloatType Z = INVARIANT_ADD(P, Q);
	const FFloatType E = INVARIANT_ADD(INVARIANT_ADD(INVARIANT_SUB(P, Z), Q), INVARIANT_MUL(Tx, Ty));
	return DFFastTwoSum(Z, E);
}

// Multiply up two single precision scalars to one double precision scalar. (AKA Fast2Mult)
// [1] Algorithm 3
FDFType DFTwoProduct(FFloatType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs * Rhs);
#elif !PLATFORM_SUPPORTS_FMA
	return DFTwoProductNoFMA(Lhs, Rhs);
#else
	const FFloatType P = Lhs * Rhs;
	const FFloatType E = INVARIANT_FMA(Lhs, Rhs, -P);
	return DFConstructor(P, E);
#endif
}

// Alias for DFTwoProduct
FDFType DFMultiply(FFloatType Lhs, FFloatType Rhs) { return DFTwoProduct(Lhs, Rhs); }

// [1] Algorithm 11.
// If no FMA is available, mad() expansion makes this equivalent to [1] Algorithm 10
FDFType DFMultiply(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High * Rhs.High);
#else
	const FDFType P = DFTwoProduct(Lhs.High, Rhs.High);
	FFloatType T = INVARIANT_MUL(Lhs.High, Rhs.Low);
	T = INVARIANT_FMA(Lhs.Low, Rhs.High, T);
	T = INVARIANT_ADD(T, P.Low);

	return DFFastTwoSum(P.High, T);
#endif
}

// [1] Algorithm 9
// If no FMA is available, mad() expansion makes this equivalent to [1] Algorithm 8
FDFType DFMultiply(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High * Rhs);
#else
	FDFType P = DFTwoProduct(Lhs.High, Rhs);
	FFloatType T = INVARIANT_FMA(Lhs.Low, Rhs, P.Low);
	return DFFastTwoSum(P.High, T);
#endif
}

FDFType DFMultiply(FFloatType Lhs, FDFType Rhs) { return DFMultiply(Rhs, Lhs); }

FFloatType DFMultiplyDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High * Rhs.High;
#else
	const FDFType P = DFTwoProduct(Lhs.High, Rhs.High);
	FFloatType T = INVARIANT_MUL(Lhs.High, Rhs.Low);
	T = INVARIANT_FMA(Lhs.Low, Rhs.High, T);
	T = INVARIANT_ADD(T, P.Low);
	const FFloatType S = INVARIANT_ADD(P.High, T);
	return S;
#endif
}

FFloatType DFMultiplyDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High * Rhs;
#else
	FDFType P = DFTwoProduct(Lhs.High, Rhs);
	P.Low = INVARIANT_FMA(Lhs.Low, Rhs, P.Low);
	const FFloatType S = INVARIANT_ADD(P.High, P.Low);
	return S;
#endif
}

FFloatType DFMultiplyDemote(FFloatType Lhs, FDFType Rhs) { return DFMultiplyDemote(Rhs, Lhs); }
FFloatType DFMultiplyDemote(FFloatType Lhs, FFloatType Rhs) { return Lhs * Rhs; }

// Fast multiplication that assumes the factor is a power of two.
// If this assumption is valid, no precision is lost.
FDFType DFMultiplyByPow2(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High * Rhs);
#else
	return DFConstructor(Lhs.High * Rhs, Lhs.Low * Rhs);
#endif
}

FDFType DFSqr(FDFType V) { return DFMultiply(V, V); }
FDFType DFSqr(FFloatType V) { return DFTwoProduct(V, V); }

/**************/
/** DIVISION **/
/**************/

FDFType DFFastDivide(FDFType Lhs, FDFType Rhs);

// [1] Algorithm 18
FDFType DFDivide(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High / Rhs.High);
#elif !PLATFORM_SUPPORTS_FMA
	return DFFastDivide(Lhs, Rhs);
#else
	const FFloatType Th = INVARIANT_DIV(1.0, Rhs.High);
	const FFloatType Rh = INVARIANT_FMA(-Rhs.High, Th, (FFloatType)1);
	const FFloatType Rl = INVARIANT_MUL(-Rhs.Low, Th);
	const FDFType E = DFFastTwoSum(Rh, Rl);
	const FDFType D = DFMultiply(E, Th);
	const FDFType M = DFAdd(D, Th);
	return DFMultiply(Lhs, M);
#endif
}

// [1] Algorithm 15
FDFType DFDivide(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High / Rhs);
#else
	const FFloatType Th = INVARIANT_DIV(Lhs.High, Rhs);
	const FDFType P = DFTwoProduct(Th, Rhs);
	const FFloatType Dh = INVARIANT_SUB(Lhs.High, P.High);
	const FFloatType Dt = INVARIANT_SUB(Dh, P.Low);
	const FFloatType D = INVARIANT_ADD(Dt, Lhs.Low);
	const FFloatType Tl = INVARIANT_DIV(D, Rhs);
	return DFFastTwoSum(Th, Tl);
#endif
}

FDFType DFDivide(FFloatType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs / Rhs);
#else
	return DFDivide(DFPromote(Lhs), Rhs);
#endif
}

FDFType DFDivide(FFloatType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs / Rhs.High);
#else
	return DFDivide(DFPromote(Lhs), Rhs);
#endif
}

// Less precise than 18, but almost half the ops
// [1] Algorithm 17
FDFType DFFastDivide(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High / Rhs.High);
#elif UE_DF_NO_FAST_MATH
	return DFDivide(Lhs, Rhs);
#else
	const FFloatType Th = INVARIANT_DIV(Lhs.High, Rhs.High);
	const FDFType R = DFMultiply(Rhs, Th);
	const FFloatType Ph = INVARIANT_SUB(Lhs.High, R.High);
	const FFloatType Dl = INVARIANT_SUB(Lhs.Low, R.Low);
	const FFloatType D = INVARIANT_ADD(Ph, Dl);
	const FFloatType Tl = INVARIANT_DIV(D, Rhs.High);
	return DFFastTwoSum(Th, Tl);
#endif
}

FDFType DFFastDivide(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High / Rhs);
#elif UE_DF_NO_FAST_MATH && PLATFORM_SUPPORTS_FMA
	return DFDivide(Lhs, Rhs);
#else
	const FFloatType Th = INVARIANT_DIV(Lhs.High, Rhs);
	const FDFType R = DFTwoProduct(Rhs, Th);
	const FFloatType Ph = INVARIANT_SUB(Lhs.High, R.High);
	const FFloatType Dl = INVARIANT_SUB(Lhs.Low, R.Low);
	const FFloatType D = INVARIANT_ADD(Ph, Dl);
	const FFloatType Tl = INVARIANT_DIV(D, Rhs);
	return DFFastTwoSum(Th, Tl);
#endif
}

FDFType DFFastDivide(FFloatType Lhs, FDFType Rhs) { return DFFastDivide(DFPromote(Lhs), Rhs); }
FDFType DFFastDivide(FFloatType Lhs, FFloatType Rhs) { return DFFastDivide(DFPromote(Lhs), Rhs); }

FFloatType DFFastDivideDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High / Rhs.High;
#elif UE_DF_NO_FAST_MATH
	return DFDemote(DFDivide(Lhs, Rhs));
#else
	const FFloatType Th = INVARIANT_DIV(Lhs.High, Rhs.High);
	const FDFType R = DFMultiply(Rhs, Th);
	const FFloatType Ph = INVARIANT_SUB(Lhs.High, R.High);
	const FFloatType Dl = INVARIANT_SUB(Lhs.Low, R.Low);
	const FFloatType D = INVARIANT_ADD(Ph, Dl);
	const FFloatType Tl = INVARIANT_DIV(D, Rhs.High);
	return Th + Tl;
#endif
}

FFloatType DFFastDivideDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High / Rhs;
#elif UE_DF_NO_FAST_MATH
	return DFDemote(DFDivide(Lhs, Rhs));
#else
	const FFloatType Th = INVARIANT_DIV(Lhs.High, Rhs);
	const FDFType R = DFTwoProduct(Rhs, Th);
	const FFloatType Ph = INVARIANT_SUB(Lhs.High, R.High);
	const FFloatType Dl = INVARIANT_SUB(Lhs.Low, R.Low);
	const FFloatType D = INVARIANT_ADD(Ph, Dl);
	const FFloatType Tl = INVARIANT_DIV(D, Rhs);
	return Th + Tl;
#endif
}

FFloatType DFFastDivideDemote(FFloatType Lhs, FDFType Rhs) { return DFFastDivideDemote(DFPromote(Lhs), Rhs); }
FFloatType DFFastDivideDemote(FFloatType Lhs, FFloatType Rhs) { return Lhs / Rhs; }

// Fast division that assumes the divisor is a power of two.
// If this assumption is valid, no precision is lost.
FDFType DFDivideByPow2(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(Lhs.High / Rhs);
#else
	return DFConstructor(Lhs.High / Rhs, Lhs.Low / Rhs);
#endif
}

FDFType DFRcp(FDFType V) { return DFFastDivide((FFloatType)1.0, V); }
FFloatType DFRcpDemote(FDFType V) { return rcp(V.High); }

/****************/
/** COMPARISON **/
/****************/

FBoolType DFEqualsApprox(FDFType Lhs, FDFType Rhs, float Threshold)
{
	return DFSubtractDemote(Lhs, Rhs) < Threshold;
}
FBoolType DFEqualsApprox(FDFType Lhs, FFloatType Rhs, float Threshold)
{
	return DFSubtractDemote(Lhs, Rhs) < Threshold;
}
FBoolType DFEqualsApprox(FFloatType Lhs, FDFType Rhs, float Threshold)
{
	return DFSubtractDemote(Lhs, Rhs) < Threshold;
}

FBoolType DFEquals(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High == Rhs.High;
#else
#if COMPILER_SUPPORTS_HLSL2021
	return and(Lhs.High == Rhs.High, Lhs.Low == Rhs.Low);
#else
	return Lhs.High == Rhs.High && Lhs.Low == Rhs.Low;
#endif
#endif
}
FBoolType DFEquals(FDFType Lhs, FFloatType Rhs)
{
	return DFEquals(Lhs, DFPromote(Rhs));
}
FBoolType DFEquals(FFloatType Lhs, FDFType Rhs)
{
	return DFEquals(DFPromote(Lhs), Rhs);
}

// Needed for new translator
FBoolType DFNotEquals(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High != Rhs.High;
#else
#if COMPILER_SUPPORTS_HLSL2021
	return or(Lhs.High != Rhs.High, Lhs.Low != Rhs.Low);
#else
	return Lhs.High != Rhs.High || Lhs.Low != Rhs.Low;
#endif
#endif
}

FDFType DFSelect(FBoolType S, FDFType Lhs, FDFType Rhs) { return DFConstructor(select(S, Lhs.High, Rhs.High), select(S, Lhs.Low, Rhs.Low)); }
FDFType DFSelect(FBoolType S, FDFType Lhs, FFloatType Rhs) { return DFConstructor(select(S, Lhs.High, Rhs), select(S, Lhs.Low, (FFloatType)0)); }
FDFType DFSelect(FBoolType S, FFloatType Lhs, FDFType Rhs) { return DFConstructor(select(S, Lhs, Rhs.High), select(S, (FFloatType)0, Rhs.Low)); }

FBoolType DFGreater(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High > Rhs.High;
#else
#if COMPILER_SUPPORTS_HLSL2021
		return or(Lhs.High > Rhs.High, and(Lhs.High == Rhs.High, Lhs.Low > Rhs.Low));
#else
		return Lhs.High > Rhs.High || (Lhs.High == Rhs.High && Lhs.Low > Rhs.Low);
#endif
#endif
}

FBoolType DFLess(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High < Rhs.High;
#else
#if COMPILER_SUPPORTS_HLSL2021
	return or(Lhs.High < Rhs.High, and(Lhs.High == Rhs.High, Lhs.Low < Rhs.Low));
#else
	return Lhs.High < Rhs.High || (Lhs.High == Rhs.High && Lhs.Low < Rhs.Low);
#endif
#endif
}

FBoolType DFGreater(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High > Rhs;
#else
#if COMPILER_SUPPORTS_HLSL2021
	return or(Lhs.High > Rhs, and(Lhs.High == Rhs, Lhs.Low > 0.0));
#else
	return Lhs.High > Rhs || (Lhs.High == Rhs && Lhs.Low > 0.0);
#endif
#endif
}

FBoolType DFLess(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return Lhs.High < Rhs;
#else
#if COMPILER_SUPPORTS_HLSL2021
	return or(Lhs.High < Rhs, and(Lhs.High == Rhs, Lhs.Low < 0.0));
#else
	return Lhs.High < Rhs || (Lhs.High == Rhs && Lhs.Low < 0.0);
#endif
#endif
}

FBoolType DFGreater(FFloatType Lhs, FDFType Rhs) { return DFLess(Rhs, Lhs); }
FBoolType DFLess(FFloatType Lhs, FDFType Rhs) { return DFGreater(Rhs, Lhs); }

FBoolType DFGreaterEqual(FDFType Lhs, FDFType Rhs) { return !DFLess(Lhs, Rhs); }
FBoolType DFGreaterEqual(FDFType Lhs, FFloatType Rhs) { return !DFLess(Lhs, Rhs); }
FBoolType DFGreaterEqual(FFloatType Lhs, FDFType Rhs) { return !DFLess(Lhs, Rhs); }

FBoolType DFLessEqual(FDFType Lhs, FFloatType Rhs) { return !DFGreater(Lhs, Rhs); }
FBoolType DFLessEqual(FDFType Lhs, FDFType Rhs) { return !DFGreater(Lhs, Rhs); }
FBoolType DFLessEqual(FFloatType Lhs, FDFType Rhs) { return !DFGreater(Lhs, Rhs); }

FDFType DFMin(FDFType Lhs, FDFType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), Lhs, Rhs);
}

FDFType DFMin(FDFType Lhs, FFloatType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), Lhs, DFPromote(Rhs));
}

FDFType DFMin(FFloatType Lhs, FDFType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), DFPromote(Lhs), Rhs);
}

FDFType DFMax(FDFType Lhs, FDFType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), Rhs, Lhs);
}

FDFType DFMax(FDFType Lhs, FFloatType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), DFPromote(Rhs), Lhs);
}

FDFType DFMax(FFloatType Lhs, FDFType Rhs)
{
	return DFSelect(DFLess(Lhs, Rhs), Rhs, DFPromote(Lhs));
}

// Needed for new translator
FDFType DFClamp(FDFType X, FDFType A, FDFType B)
{
	return DFMin(DFMax(X, A), B);
}

/**********/
/** MISC **/
/**********/

struct FDFTypeDeriv
{
	FDFType Value;
	FFloatType Ddx;
	FFloatType Ddy;
};

// [2] Algorithm 8
FDFType DFSqrt(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(sqrt(V.High));
#else
	const FFloatType Sh = sqrt(V.High);
	const FFloatType P1 = INVARIANT_FMA(-Sh, Sh, V.High); // Note: precision loss here if FMA is not supported
	const FFloatType P2 = INVARIANT_ADD(V.Low, P1);

	const FFloatType Sl = INVARIANT_DIV(P2, (2.0 * Sh));
	return DFFastTwoSum(Sh, Sl);

	// equivalent to (remark 3.5)
	//precise FFloatType Tl = P2 / Sh;
	//precise FFloatType Zh = FMA((FFloatType)0.5, Tl, Sh);
	//precise FFloatType D = Zh - Sh;
	//precise FFloatType Zl = FMA((FFloatType)0.5, Tl, -D);
	//return DFConstructor(Zh, Zl);
#endif
}

// [2] Algorithm 9
FFloatType DFSqrtDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return sqrt(V.High);
#else
	const FFloatType Sh = sqrt(V.High);
	const FFloatType P1 = INVARIANT_FMA(-Sh, Sh, V.High); // Note: precision loss here if FMA is not supported
	const FFloatType P2 = INVARIANT_ADD(V.Low, P1);

	const FFloatType Sl = INVARIANT_DIV(P2, (2.0 * Sh));
	return Sh + Sl;
#endif
}

FDFType DFRsqrt(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(rsqrt(V.High));
#else
	return DFDivide((FFloatType)1.0, DFSqrt(V));
#endif
}
FFloatType DFRsqrtDemote(FDFType V) { return rsqrt(V.High); }

FFloatType DFSin(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return sin(V.High);
#else
	FFloatType Sh, Ch;
	sincos(V.High, Sh, Ch);
	FFloatType Sl, Cl;
	sincos(V.Low, Sl, Cl);
	return Sh*Cl + Ch*Sl; // trigonometric identity for sin(h+l)
#endif
}

FFloatType DFCos(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return cos(V.High);
#else
	FFloatType Sh, Ch;
	sincos(V.High, Sh, Ch);
	FFloatType Sl, Cl;
	sincos(V.Low, Sl, Cl);
	return Ch*Cl - Sh*Sl; // trigonometric identity for cos(h+l)
#endif
}

void DFSinCos(FDFType V, out FFloatType Sin, out FFloatType Cos)
{
#if UE_DF_FORCE_FP32_OPS
	sincos(V.High, Sin, Cos);
#else
	FFloatType Sh, Ch;
	sincos(V.High, Sh, Ch);
	FFloatType Sl, Cl;
	sincos(V.Low, Sl, Cl);
	Sin = FMA(Sh, Cl,  Ch*Sl); // trigonometric identity for sin(h+l)
	Cos = FMA(Ch, Cl, -Sh*Sl); // trigonometric identity for cos(h+l)
#endif
}

FFloatType DFTan(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return tan(V.High);
#else
	FFloatType Th = tan(V.High);
	FFloatType Tl = tan(V.Low);
	return (Th + Tl) / (1.0 - Th * Tl); // trigonometric identity for tan(h+l)
#endif
}

FFloatType DFAsin(FDFType V)
{
	return asin(DFDemote(V));
}

FFloatType DFAcos(FDFType V)
{
	return acos(DFDemote(V));
}

FFloatType DFAtan(FDFType V)
{
	return atan(DFDemote(V));
}

FFloatType DFModf(FDFType V, out FDFType Integer)
{
#if UE_DF_FORCE_FP32_OPS
	Integer.Low = 0;
	return modf(V.High, Integer.High);
#else
	FFloatType IntHigh;
	FFloatType FracHigh = modf(V.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(V.Low, IntLow);
	FFloatType IntSum;
	FFloatType FracSum = modf(FracHigh + FracLow, IntSum);
	Integer = DFTwoSum(IntHigh, IntLow + IntSum);
	return FracSum;
#endif
}

FFloatType DFModfDemote(FDFType V, out FFloatType Integer)
{
#if UE_DF_FORCE_FP32_OPS
	return modf(V.High, Integer);
#else
	FFloatType IntHigh;
	FFloatType FracHigh = modf(V.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(V.Low, IntLow);
	FFloatType IntSum;
	FFloatType FracSum = modf(FracHigh + FracLow, IntSum);
	Integer = IntHigh + IntLow + IntSum;
	return FracSum;
#endif
}

FDFType DFCeil(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(ceil(V.High));
#else
	FFloatType IntHigh;
	FFloatType FracHigh = modf(V.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(V.Low, IntLow);
	IntLow += ceil(FracHigh + FracLow);
	return DFTwoSum(IntHigh, IntLow);
#endif
}

FDFType DFFloor(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(floor(V.High));
#else
	FFloatType IntHigh;
	FFloatType FracHigh = modf(V.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(V.Low, IntLow);
	IntLow += floor(FracHigh + FracLow);
	return DFTwoSum(IntHigh, IntLow);
#endif
}

FDFType DFRound(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(round(V.High));
#else
	FFloatType Sign = DFSign(V);
	FDFType VAbs = DFConstructor(Sign*V.High, Sign*V.Low);

	FFloatType IntHigh;
	FFloatType FracHigh = modf(VAbs.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(VAbs.Low, IntLow);

	IntLow += floor(FracHigh + FracLow + 0.5);
	return DFTwoSum(Sign * IntHigh, Sign * IntLow);
#endif
}

FDFType DFTrunc(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(trunc(V.High));
#else
	FFloatType Sign = DFSign(V);
	FDFType VAbs = DFConstructor(Sign*V.High, Sign*V.Low);

	FFloatType IntHigh;
	FFloatType FracHigh = modf(VAbs.High, IntHigh);
	FFloatType IntLow;
	FFloatType FracLow = modf(VAbs.Low, IntLow);

	IntLow += floor(FracHigh + FracLow);
	return DFTwoSum(Sign * IntHigh, Sign * IntLow);
#endif
}

FDFType DFFrac(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(frac(V.High));
#else
	return DFSubtract(V, DFFloor(V));
#endif
}

// Technically, this function has very low precision due to discontinuity+rounding
// (e.g. 0.99.. ~= 1.0 => frac(0.99..) = frac(1.0) = 0.0)
// but this is irrelevant in practical cases
// TODO: test negative values
FFloatType DFFracDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return frac(V.High);
#else
	return frac(frac(V.High) + frac(V.Low));
#endif
}

// Similar to HLSL fmod, this is equivalent to `lhs - rhs * trunc(lhs / rhs)`
// so fmod(-0.1, 1.0) returns -0.1, not 0.9 (as mod does in GLSL, which is the true mathematical modulo operator)
FDFType DFFmod(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(fmod(Lhs.High, Rhs));
#else
	return DFSubtract(Lhs, DFMultiply(DFTrunc(DFDivide(Lhs, Rhs)), Rhs));
#endif
}

FFloatType DFFmodDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return fmod(Lhs.High, Rhs);
#else
	//DF_TODO: optimize. Is equal to frac(Lhs/Rhs)*Rhs?
	//not equal to fmod(fmod(high)+fmod(low)) due to how fmod handles negative values
	return DFSubtractDemote(Lhs, DFMultiply(DFTrunc(DFDivide(Lhs, Rhs)), Rhs)); 
#endif
}

FFloatType DFFmodByPow2Demote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return fmod(Lhs.High, Rhs);
#else
	return DFFracDemote(DFDivideByPow2(Lhs, Rhs)) * Rhs;
#endif
}

FDFType DFLerp(FDFType Lhs, FDFType Rhs, FFloatType S)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(lerp(Lhs.High, Rhs.High, S));
#else
	return DFTwoSum(lerp(Lhs.High, Rhs.High, S), lerp(Lhs.Low, Rhs.Low, S)); //DF_TODO: precision issues
#endif
}

FDFType DFSaturate(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(saturate(V.High));
#else
	V = DFSelect(DFLess(V, 1.0), V, 1.0);
	V = DFSelect(DFLess(V, 0.0), 0.0, V);
	return V;
#endif
}
FFloatType DFSaturateDemote(FDFType V) { return saturate(DFDemote(V)); }

FDFType DFSmoothStep(FDFType Lhs, FDFType Rhs, FDFType S)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(smoothstep(Lhs.High, Rhs.High, S.High));
#else
	FDFType T = DFSaturate(DFDivide(DFSubtract(S, Lhs), DFSubtract(Rhs, Lhs)));
	return DFMultiply(DFSqr(T), DFSubtract(3.0f, DFMultiplyByPow2(T, 2.0f)));
#endif
}
FFloatType DFSmoothStepDemote(FDFType Lhs, FDFType Rhs, FDFType S)
{
#if UE_DF_FORCE_FP32_OPS
	return smoothstep(Lhs.High, Rhs.High, S.High);
#else
	FFloatType T = DFSaturateDemote(DFDivide(DFSubtract(S, Lhs), DFSubtract(Rhs, Lhs)));
	return T*T*(3.0f - (2.0f*T));
#endif
}

FFloatType DFStep(FDFType Lhs, FDFType Rhs)    { return select(DFGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }
FFloatType DFStep(FDFType Lhs, FFloatType Rhs) { return select(DFGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }
FFloatType DFStep(FFloatType Lhs, FDFType Rhs) { return select(DFGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }

// No ddxy inside ray tracing shaders
#if RAYHITGROUPSHADER || RAYMISSHADER || RAYCALLABLESHADER || USE_FORCE_TEXTURE_MIP
FDFType DFDdx(FDFType V) { return (FDFType)0; }
FDFType DFDdy(FDFType V) { return (FDFType)0; }
FFloatType DFDdxDemote(FDFType V) { return (FFloatType)0; }
FFloatType DFDdyDemote(FDFType V) { return (FFloatType)0; }
#else

FDFType DFDdx(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(ddx(V.High));
#else
	return DFTwoSum(ddx(V.High), ddx(V.Low));
#endif
}

FDFType DFDdy(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(ddy(V.High));
#else
	return DFTwoSum(ddy(V.High), ddy(V.Low));
#endif
}

FFloatType DFDdxDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return ddx(V.High);
#else
	return ddx(V.High) + ddx(V.Low);
#endif
}

FFloatType DFDdyDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return ddy(V.High);
#else
	return ddy(V.High) + ddy(V.Low);
#endif
}
#endif
