// Copyright Epic Games, Inc. All Rights Reserved.

#if COMPILER_SUPPORTS_HLSL2021

#define DECLARE_VEC_VEC_OP(DFOperation) \
	FDFType DFOperation(FDFType Lhs, FDFType Rhs); \
	FDFType DFOperation(FDFType Lhs, FFloatType Rhs); \
	FDFType DFOperation(FDFType Lhs, FDFScalar Rhs); \
	FDFType DFOperation(FDFType Lhs, float Rhs);

struct FDFType;
FDFType DFBroadcastFunc(FDFScalar V);
FFloatType DFBroadcastFunc(float V);
FDFScalar DFGetComponent(FDFType V, int C);
FDFType DFNegate(FDFType In);
DECLARE_VEC_VEC_OP(DFAdd);
DECLARE_VEC_VEC_OP(DFSubtract);
DECLARE_VEC_VEC_OP(DFMultiply);
DECLARE_VEC_VEC_OP(DFDivide);
FBoolType DFGreater(FDFType Lhs, FFloatType Rhs);
FBoolType DFLess(FDFType Lhs, FFloatType Rhs);
FFloatType DFDemote(FDFType In);

#undef DECLARE_VEC_VEC_OP

#endif

struct FDFType
{
	FFloatType High;
	FFloatType Low;

#if COMPILER_SUPPORTS_HLSL2021

#define DECLARE_VEC_VEC_OP(Operator, DFOperation) \
	FDFType Operator(FDFType Rhs) { return DFOperation(this, Rhs); } \
	FDFType Operator(FFloatType Rhs) { return DFOperation(this, Rhs); } \
	FDFType Operator(FDFScalar Rhs) { return DFOperation(this, Rhs); } \
	FDFType Operator(float Rhs) { return DFOperation(this, Rhs); }

	//Commented out until supported by HlslParser
	//FDFScalar operator[](int idx) { return DFGetComponent(this, idx); }
	//FDFType operator-() { return DFNegate(this); }
	////DECLARE_VEC_VEC_OP(operator+, DFAdd)
	////DECLARE_VEC_VEC_OP(operator-, DFSubtract)
	////DECLARE_VEC_VEC_OP(operator*, DFMultiply)
	////DECLARE_VEC_VEC_OP(operator/, DFDivide)
	//FBoolType operator>(FFloatType Rhs) { return DFGreater(this, Rhs); }
	//FBoolType operator<(FFloatType Rhs) { return DFLess(this, Rhs); }

	FFloatType Demote() { return DFDemote(this); }

#undef DECLARE_VEC_VEC_OP

#endif
};

FDFScalar DFGetComponent(FDFType V, int C) { return MakeDFScalar(V.High[C], V.Low[C]); }

#include "DoubleFloatOperations.ush"

FDFType DFBroadcastFunc(FDFScalar V)
{
	const FFloatType High = V.High;
	const FFloatType Low = V.Low;
	return DFConstructor(High, Low);
}

FFloatType DFBroadcastFunc(float V)
{
	const FFloatType Single = V;
	return Single;
}

#define DEFINE_BROADCASTED_OP(FReturnType, DFOperation) \
	FReturnType DFOperation(FDFScalar Lhs, FDFType Rhs) { return DFOperation(DFBroadcastFunc(Lhs), Rhs); } \
	FReturnType DFOperation(float Lhs, FDFType Rhs) { return DFOperation(DFBroadcastFunc(Lhs), Rhs); } \
	FReturnType DFOperation(FDFScalar Lhs, FFloatType Rhs) { return DFOperation(DFBroadcastFunc(Lhs), Rhs); } \
	FReturnType DFOperation(float Lhs, FFloatType Rhs) { return DFOperation(DFBroadcastFunc(Lhs), Rhs); } \
	FReturnType DFOperation(FDFType Lhs, FDFScalar Rhs) { return DFOperation(Lhs, DFBroadcastFunc(Rhs)); } \
	FReturnType DFOperation(FDFType Lhs, float Rhs)     { return DFOperation(Lhs, DFBroadcastFunc(Rhs)); } \
	FReturnType DFOperation(FFloatType Lhs, FDFScalar Rhs) { return DFOperation(Lhs, DFBroadcastFunc(Rhs)); } \
	FReturnType DFOperation(FFloatType Lhs, float Rhs) { return DFOperation(Lhs, DFBroadcastFunc(Rhs)); }

DEFINE_BROADCASTED_OP(FDFType	, DFAdd)
DEFINE_BROADCASTED_OP(FFloatType, DFAddDemote)
DEFINE_BROADCASTED_OP(FDFType	, DFFastAdd)
DEFINE_BROADCASTED_OP(FFloatType, DFFastAddDemote)
DEFINE_BROADCASTED_OP(FDFType	, DFSubtract)
DEFINE_BROADCASTED_OP(FFloatType, DFSubtractDemote)
DEFINE_BROADCASTED_OP(FDFType	, DFFastSubtract)
DEFINE_BROADCASTED_OP(FFloatType, DFFastSubtractDemote)
DEFINE_BROADCASTED_OP(FDFType	, DFMultiply)
DEFINE_BROADCASTED_OP(FFloatType, DFMultiplyDemote)
DEFINE_BROADCASTED_OP(FDFType	, DFDivide)
DEFINE_BROADCASTED_OP(FDFType	, DFFastDivide)
DEFINE_BROADCASTED_OP(FFloatType, DFFastDivideDemote)


#undef DEFINE_BROADCASTED_OP

FDFType DFDivideByPow2(FDFType Lhs, float Rhs)
{
	FFloatType RhsVect = Rhs;
	return DFDivideByPow2(Lhs, RhsVect);
}

FDFScalar DFVectorSum(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	float Result = V.High[0] + V.High[1];
	[unroll]
	for (int i = 2; i < GNumComponents; ++i)
	{
		Result = Result + V.High[i];
	}
	return DFPromote(Result);
#else
	FDFScalar Result = DFAdd(DFGetComponent(V, 0), DFGetComponent(V, 1));
	[unroll]
	for (int i = 2; i < GNumComponents; ++i)
	{
		Result = DFAdd(Result, DFGetComponent(V, i));
	}
	return Result;
#endif
}

FDFScalar DFFastVectorSum(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFVectorSum(V);
#else
	FDFScalar Result = DFFastAdd(DFGetComponent(V, 0), DFGetComponent(V, 1));
	[unroll]
	for (int i = 2; i < GNumComponents; ++i)
	{
		Result = DFFastAdd(Result, DFGetComponent(V, i));
	}
	return Result;
#endif
}

FDFScalar DFDot(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(dot(Lhs.High, Rhs));
#else
	return DFFastVectorSum(DFMultiply(Lhs, Rhs));
#endif
}
FDFScalar DFDot(FFloatType Lhs, FDFType Rhs) { return DFDot(Rhs, Lhs); }
FDFScalar DFDot(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(dot(Lhs.High, Rhs.High));
#else
	return DFFastVectorSum(DFMultiply(Lhs, Rhs));
#endif
}

FFloatType DFDotDemote(FDFType Lhs, FFloatType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return dot(Lhs.High, Rhs);
#else
	return DFDemote(DFFastVectorSum(DFMultiply(Lhs, Rhs)));
#endif
}
FFloatType DFDotDemote(FFloatType Lhs, FDFType Rhs) { return DFDotDemote(Rhs, Lhs); }
FFloatType DFDotDemote(FDFType Lhs, FDFType Rhs)
{
#if UE_DF_FORCE_FP32_OPS
	return dot(Lhs.High, Rhs.High);
#else
	return DFDemote(DFFastVectorSum(DFMultiply(Lhs, Rhs)));
#endif
}

FDFScalar DFLengthSqr(FFloatType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(dot(V, V));
#else
	return DFFastVectorSum(DFSqr(V));
#endif
}

FDFScalar DFLength(FFloatType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(length(V));
#else
	return DFSqrt(DFLengthSqr(V));
#endif
}

// DF_TODO: test potential precision issues

FDFScalar DFLengthSqr(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(dot(V.High, V.High));
#else
	return DFFastVectorSum(DFSqr(V));
#endif
}
FDFScalar DFLength(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(length(V.High));
#else
	return DFSqrt(DFLengthSqr(V));
#endif
}

float DFLengthDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return length(V.High);
#else
	return DFSqrtDemote(DFLengthSqr(V));
#endif
}

float DFRcpLengthDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return rcp(length(V.High));
#else
	return DFRsqrtDemote(DFLengthSqr(V));
#endif
}

FDFType DFNormalize(FFloatType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(normalize(V));
#else
	return DFDivide(V, DFBroadcastFunc(DFLength(V)));
#endif
}

FDFType DFNormalize(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return DFPromote(normalize(V.High));
#else
	return DFFastDivide(V, DFLength(V));
#endif
}

FFloatType DFNormalizeDemote(FDFType V)
{
#if UE_DF_FORCE_FP32_OPS
	return normalize(V.High);
#else
	return V.High / length(V.High);
#endif
}
