// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DownSampleDepthPixelShader.usf: Downsamples scene depth by a factor of 2.
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "SceneTexturesCommon.ush"
#if NANITE_COMPOSITE
#include "Nanite/NaniteDataDecode.ush"
#endif

// This must match EDownsampleDepthFilter
#define DOWNSAMPLE_DEPTH_FILTER_POINT		0
#define DOWNSAMPLE_DEPTH_FILTER_MAX			1
#define DOWNSAMPLE_DEPTH_FILTER_CBMINMAX	2

Texture2D<uint> NaniteShadingMask;
Texture2D<float> DepthTexture;
#if OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH
Texture2D<float2> MinMaxDepthTexture;
#endif
float2 DstToSrcPixelScale;
float2 SourceMaxUV;
float2 DestinationResolution;
uint DownsampleDepthFilter;

// Only valid when using OUTPUT_MIN_AND_MAX_DEPTH
int4 DstPixelCoordMinAndMax;
int4 SrcPixelCoordMinAndMax;

float GetDeviceZ(float2 UV)
{
	return DepthTexture.Sample(GlobalPointClampedSampler, min(UV, SourceMaxUV));
}

void Main(
	noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	in FStereoPSInput StereoInput,
	in float4 SVPos : SV_POSITION,
#if OUTPUT_MIN_AND_MAX_DEPTH || OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH
	out float4 OutColor : SV_Target0
#else
	out float OutDepth : SV_DEPTH
#endif
)
{
	// Pixel coordinate for the output 
	const int2 DstPixelPos = SVPos.xy;
	// Lower left corner from the input 2x2 quad
	const int2 PixelPos = float2(DstPixelPos) * DstToSrcPixelScale;

#if OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH

	float2 MinMaxDepth0 = MinMaxDepthTexture.Load(uint3(clamp(PixelPos + int2(0, 0), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float2 MinMaxDepth1 = MinMaxDepthTexture.Load(uint3(clamp(PixelPos + int2(0, 1), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float2 MinMaxDepth2 = MinMaxDepthTexture.Load(uint3(clamp(PixelPos + int2(1, 1), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float2 MinMaxDepth3 = MinMaxDepthTexture.Load(uint3(clamp(PixelPos + int2(1, 0), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));

	const float MaxDeviceZ = max(max(MinMaxDepth0.y, MinMaxDepth1.y), max(MinMaxDepth2.y, MinMaxDepth3.y));
	const float MinDeviceZ = min(min(MinMaxDepth0.x, MinMaxDepth1.x), min(MinMaxDepth2.x, MinMaxDepth3.x));
	OutColor = float4(MinDeviceZ, MaxDeviceZ, 0.0f, 0.0f);

#else // OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH

	float DeviceZ0 = DepthTexture.Load(uint3(clamp(PixelPos + int2(0, 0), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float DeviceZ1 = DepthTexture.Load(uint3(clamp(PixelPos + int2(0, 1), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float DeviceZ2 = DepthTexture.Load(uint3(clamp(PixelPos + int2(1, 1), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));
	float DeviceZ3 = DepthTexture.Load(uint3(clamp(PixelPos + int2(1, 0), SrcPixelCoordMinAndMax.xy, SrcPixelCoordMinAndMax.zw), 0));

#if OUTPUT_MIN_AND_MAX_DEPTH

	const float MaxDeviceZ = max(max(DeviceZ0, DeviceZ1), max(DeviceZ2, DeviceZ3));
	const float MinDeviceZ = min(min(DeviceZ0, DeviceZ1), min(DeviceZ2, DeviceZ3));
	OutColor = float4(MinDeviceZ, MaxDeviceZ, 0.0f, 0.0f);

#else // OUTPUT_MIN_AND_MAX_DEPTH

	float Depth = 0;

	if (DownsampleDepthFilter < DOWNSAMPLE_DEPTH_FILTER_CBMINMAX)
	{
		float FarDepth = min(min(DeviceZ0, DeviceZ1), min(DeviceZ2, DeviceZ3));

		// Max depth shrinks the silhouettes around foreground objects and is conservative for depth testing
		// Sample 0 has consistent error, use whichever one is requested for this downsample
		Depth = DownsampleDepthFilter == DOWNSAMPLE_DEPTH_FILTER_MAX ? FarDepth : DeviceZ0;
	}
	else // DownsampleDepthFilter == DOWNSAMPLE_DEPTH_FILTER_CBMINMAX
	{
		const float MaxDeviceZ = max(max(DeviceZ0, DeviceZ1), max(DeviceZ2, DeviceZ3));
		const float MinDeviceZ = min(min(DeviceZ0, DeviceZ1), min(DeviceZ2, DeviceZ3));

		const uint2 PixelPosStep = (DstPixelPos >> 1) * 2;
		uint CheckerBoard = (DstPixelPos.x - PixelPosStep.x);									// horizontal alternance of black and white
		CheckerBoard = (DstPixelPos.y - PixelPosStep.y) == 0 ? CheckerBoard : 1 - CheckerBoard;// vertical toggle of horizontal checker on odd lines

		Depth = CheckerBoard > 0 ? MaxDeviceZ : MinDeviceZ;
	}

	OutDepth = Depth;

#endif // OUTPUT_MIN_AND_MAX_DEPTH

#endif // OUTPUT_MINMAXDEPTH_FROM_MINMAXDEPTH
}

#ifndef STENCIL_LIGHTING_CHANNELS_SHIFT 
#define STENCIL_LIGHTING_CHANNELS_SHIFT 0
#endif

// Must match C++
#define STENCIL_CAST_SHADOW_BIT						1
#define STENCIL_RAY_TRACING_REPRESENTATION_BIT_ID	2

uint bStencilHasCastShadowBit;

void CopyStencilToLightingChannelsPS(  
	noperspective float4 InUV : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out uint4 OutValue : SV_Target0
	)
{
	uint2 IntUV = (uint2)((float2)InUV.xy * (float2)View.BufferSizeAndInvSize.xy);

	uint LightingChannels = 0;
	bool bHasRaytracingRepresentation = true;
	bool bCastShadow = true;

#if NANITE_COMPOSITE
	FShadingMask UnpackedMask = UnpackShadingMask(NaniteShadingMask.Load(uint3(IntUV, 0)));
	BRANCH
	if (UnpackedMask.bIsNanitePixel)
	{
		LightingChannels = UnpackedMask.LightingChannels;
		bHasRaytracingRepresentation = UnpackedMask.bHasRayTracingRepresentation;
		bCastShadow = UnpackedMask.bCastShadow;
	}
	else
#endif
	{
		const uint Stencil = SceneStencilTexture.Load(uint3(IntUV, 0)) STENCIL_COMPONENT_SWIZZLE;
		const uint ShiftedStencil = Stencil >> STENCIL_LIGHTING_CHANNELS_SHIFT;
		// Flip the lowest channel bit, it was stored inverted so we can clear stencil to 0 as a default
		LightingChannels = (ShiftedStencil & 0x6) | (~ShiftedStencil & 0x1);

		bHasRaytracingRepresentation = (Stencil >> STENCIL_RAY_TRACING_REPRESENTATION_BIT_ID) & 0x1;

		if (bStencilHasCastShadowBit)
		{
			bCastShadow = (Stencil >> STENCIL_CAST_SHADOW_BIT) & 0x1;
		}
	}

	uint CombinedMask = LightingChannels;
	CombinedMask |= bCastShadow ? (1u << LIGHTING_CHANNELS_TEXTURE_CAST_SHADOWS_BIT) : 0;
	CombinedMask |= bHasRaytracingRepresentation ? (1u << LIGHTING_CHANNELS_TEXTURE_RAY_TRACING_REPRESENTATION_BIT) : 0;

	OutValue = CombinedMask;
}
