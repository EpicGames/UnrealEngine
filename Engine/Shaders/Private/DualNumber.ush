// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template< typename T >
struct TDual
{
	T Value;
	T Value_dx;
	T Value_dy;
	TDual<T> operator+(T B)
	{
		TDual<T> R;
		R.Value = Value + B;
		R.Value_dx = Value_dx;
		R.Value_dy = Value_dy;
		return R;
	}

	TDual<T> operator-(T B)
	{
		TDual<T> R;
		R.Value = Value - B;
		R.Value_dx = Value_dx;
		R.Value_dy = Value_dy;
		return R;
	}

	TDual<T> operator*(T B)
	{
		TDual<T> R;
		R.Value = Value * B;
		R.Value_dx = Value_dx * B;
		R.Value_dy = Value_dy * B;
		return R;
	}

	TDual<T> operator+(TDual<T> B)
	{
		TDual<T> R;
		R.Value = Value + B.Value;
		R.Value_dx = Value_dx + B.Value_dx;
		R.Value_dy = Value_dy + B.Value_dy;
		return R;
	}

	TDual<T> operator-(TDual<T> B)
	{
		TDual<T> R;
		R.Value = Value - B.Value;
		R.Value_dx = Value_dx - B.Value_dx;
		R.Value_dy = Value_dy - B.Value_dy;
		return R;
	}

	TDual<T> operator*(TDual<T> B)
	{
		TDual<T> R;
		R.Value = Value * B.Value;
		R.Value_dx = Value_dx * B.Value + Value * B.Value_dx;
		R.Value_dy = Value_dy * B.Value + Value * B.Value_dy;
		return R;
	}

	TDual<T> operator/(TDual<T> B)
	{
		TDual<T> R;
		R.Value = Value / B.Value;
		T Denom = rcp(B.Value * B.Value);
		T dFdA = B.Value * Denom;
		T dFdB = -Value * Denom;
		R.Value_dx = dFdA * Value_dx + dFdB * B.Value_dx;
		R.Value_dy = dFdA * Value_dy + dFdB * B.Value_dy;
		return R;
	}
};
