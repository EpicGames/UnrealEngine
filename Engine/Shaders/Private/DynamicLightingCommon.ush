// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicLightingCommon.usf: Contains functions shared by dynamic light shaders.
=============================================================================*/

#pragma once

#include "LargeWorldCoordinates.ush"

float3 GetDeferredLightTranslatedWorldPosition()
{
	return DeferredLightUniforms.TranslatedWorldPosition;
}

/** 
 * Returns a radial attenuation factor for a point light.  
 * WorldLightVector is the vector from the position being shaded to the light, divided by the radius of the light. 
 */
float RadialAttenuationMask(float3 WorldLightVector)
{
	float NormalizeDistanceSquared = dot(WorldLightVector, WorldLightVector);
	return 1.0f - saturate(NormalizeDistanceSquared);
}
float RadialAttenuation(float3 WorldLightVector, half FalloffExponent)
{
	// Old (fast, but now we not use the default of 2 which looks quite bad):
	return pow(RadialAttenuationMask(WorldLightVector), FalloffExponent);

	// New (more physically correct but slower and has a more noticable cutoff ring in the dark):
	// AttenFunc(x) = 1 / (x * x + 1)
	// derived: InvAttenFunc(y) = sqrtf(1 / y - 1)
	// FalloffExponent is ignored
	// the following code is a normalized (scaled and biased f(0)=1 f(1)=0) and optimized
/*
	// light less than x % is considered 0
	// 20% produces a bright sphere, 5 % is ok for performance, 8% looks close to the old one, smaller numbers would be more realistic but then the attenuation radius also should be increased.
	// we can expose CutoffPercentage later, alternatively we also can compute the attenuation radius from the CutoffPercentage and the brightness
	const float CutoffPercentage = 5.0f;  
	    
	float CutoffFraction = CutoffPercentage * 0.01f;  

	// those could be computed on C++ side
	float PreCompX = 1.0f - CutoffFraction;
	float PreCompY = CutoffFraction;
	float PreCompZ = CutoffFraction / PreCompX;

	return (1 / ( NormalizeDistanceSquared * PreCompX + PreCompY) - 1) * PreCompZ;
*/
}

/** 
 * Calculates attenuation for a spot light.
 * L normalize vector to light. 
 * SpotDirection is the direction of the spot light.
 * SpotAngles.x is CosOuterCone, SpotAngles.y is InvCosConeDifference. 
 */
float SpotAttenuationMask(float3 L, float3 SpotDirection, float2 SpotAngles)
{
	return saturate((dot(L, -SpotDirection) - SpotAngles.x) * SpotAngles.y);
}
float SpotAttenuation(float3 L, float3 SpotDirection, float2 SpotAngles)
{
	float ConeAngleFalloff = Square(SpotAttenuationMask(L, SpotDirection, SpotAngles));
	return ConeAngleFalloff;
}

float3 GetNormalizedLightVector(float3 TranslatedWorldPosition)
{
	// assumed to be normalized
	float3 Ret = DeferredLightUniforms.Direction;

	#if RADIAL_ATTENUATION
		Ret = normalize(GetDeferredLightTranslatedWorldPosition() - TranslatedWorldPosition);
	#endif

	return Ret;
}

float GetLightInfluenceMask(float3 TranslatedWorldPosition)
{
	float LightMask = 1;

	if (DeferredLightUniforms.InvRadius > 0)
	{
		float3 ToLight = GetDeferredLightTranslatedWorldPosition() - TranslatedWorldPosition;
		float DistanceSqr = dot(ToLight, ToLight);
		float3 L = ToLight * rsqrt(DistanceSqr);

		if (DeferredLightUniforms.FalloffExponent == 0)
		{
			LightMask = saturate(1 - Square(DistanceSqr * Square(DeferredLightUniforms.InvRadius)));
			//LightRadiusMask = Square(LightRadiusMask); No need to square since we are only doing a binary comparison below (and a saturate is used)
		}
		else
		{
			LightMask = RadialAttenuationMask(ToLight * DeferredLightUniforms.InvRadius);
		}

		if (DeferredLightUniforms.SpotAngles.x > -2.0f)
		{
			LightMask *= SpotAttenuationMask(L, -DeferredLightUniforms.Direction, DeferredLightUniforms.SpotAngles);
		}
	}

	return LightMask > 0.0f ? 1.0f : 0.0f;
}
