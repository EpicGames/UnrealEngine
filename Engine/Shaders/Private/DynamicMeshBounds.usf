// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ComputeShaderUtils.ush"

int							MaxNumToClear;
StructuredBuffer<uint>		SlotsToClearMask;
RWStructuredBuffer<int4>	OutBoundsBufferUAV;

[numthreads(64, 1, 1)]
void InitDynamicMeshBoundsCS(uint3 GroupID : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint DynamicBoundsOffset = GetUnWrappedDispatchThreadId(GroupID, GroupIndex, 64);
	if (DynamicBoundsOffset >= MaxNumToClear)
	{
		return;
	}
	
	uint WordIndex = DynamicBoundsOffset / 32u;
	uint BitIndex = DynamicBoundsOffset % 32u;
	if (SlotsToClearMask[WordIndex] & (1u << BitIndex))
	{
		const int MaxInt32 = 0x7fffffff;
		const int MinInt32 = 0x80000000;
		OutBoundsBufferUAV[DynamicBoundsOffset * 2]		= int4(MaxInt32, MaxInt32, MaxInt32, 0);
		OutBoundsBufferUAV[DynamicBoundsOffset * 2 + 1] = int4(MinInt32, MinInt32, MinInt32, 0);
	}
}

