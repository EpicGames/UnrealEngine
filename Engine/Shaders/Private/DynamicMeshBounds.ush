// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "Common.ush"

#ifndef ENABLE_DYNAMIC_MESH_BOUNDS
	#define ENABLE_DYNAMIC_MESH_BOUNDS 0
#endif

StructuredBuffer<int4> DynamicMeshBoundsBuffer;
int DynamicMeshBoundsMax;

bool LoadDynamicMeshBounds(int DynamicMeshBoundsIndex, inout float3 LocalBoundsCenter, inout float3 LocalBoundsExtent)
{
#if ENABLE_DYNAMIC_MESH_BOUNDS && COMPILER_SUPPORTS_WAVE_VOTE
	// If this draw command has dynamic bounds, load those and use instead.
	if (DynamicMeshBoundsIndex >= 0 && DynamicMeshBoundsIndex < DynamicMeshBoundsMax)
	{
		float3 DynMin = float3(DynamicMeshBoundsBuffer[DynamicMeshBoundsIndex * 2 + 0].xyz);
		float3 DynMax = float3(DynamicMeshBoundsBuffer[DynamicMeshBoundsIndex * 2 + 1].xyz);

		if (DynMin.x <= DynMax.x)
		{
			LocalBoundsCenter = (DynMin + DynMax) * 0.5f;
			LocalBoundsExtent = (DynMax - DynMin) * 0.5f;
			return true;
		}
	}
#endif
	return false;
}