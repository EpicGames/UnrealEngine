// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ETCCompressionCommon.ush:
	Helpers for compute shader ETC2 texture compression
=============================================================================*/

#pragma once

#include "GammaCorrectionCommon.ush"

#define BLOCK_MODE_INDIVIDUAL				0
#define BLOCK_MODE_DIFFERENTIAL				1u
#define NUM_RGB_TABLES 8
#define NUM_ALPHA_TABLES 16

float EtcLuminance(in float3 c)
{
	return dot(c, float3(0.3f, 0.5f, 0.2f));
}

uint EtcPackColorBits(in float3 FloatColor)
{
	uint3 ColorScaled = uint3(round(saturate(FloatColor) * 255.f));
	uint ColorPacked = (ColorScaled.r & 0x000000f8) | ((ColorScaled.g & 0x000000f8) << 8u) | ((ColorScaled.b & 0x000000f8) << 16u);
	return ColorPacked;
}

uint EtcSwapEndian32(in uint x)
{
	return
		((x & 0x0000ff) << 24u) |
		((x & 0x00ff00) <<  8u) |
		((x & 0xff0000) >>  8u) |
		(x >> 24u);
}

// Get a single scale factor to use for a YCoCg color block
// This increases precision at the expense of potential blending artifacts across blocks
float EtcGetYCoCgScale(float2 MinCoCg, float2 MaxCoCg)
{
	MinCoCg = abs(MinCoCg - 128.f / 255.f);
	MaxCoCg = abs(MaxCoCg - 128.f / 255.f);

	float MaxComponent = max(max(MinCoCg.x, MinCoCg.y), max(MaxCoCg.x, MaxCoCg.y));

	return (MaxComponent < 32.f / 255.f) ? 4.f : (MaxComponent < 64.f / 255.f) ? 2.f : 1.f;
}

void EtcApplyYCoCgScale(inout float2 CoCg, float Scale)
{
	CoCg = (CoCg - 128.f / 255.f) * Scale + 128.f / 255.f;
}

uint EtcGetBlockWeights(in float3 Block[16], in float3 BaseColor, in uint TableIdx)
{
	// weights
	// 3 - big negative distance
	// 2 - small negative distance
	// 1 - big positive distance
	// 0 - small positive distance
	uint BlockWeights = 0;

	float Distance0 = EtcParameters.RGB_DISTANCE_TABLES[TableIdx].z / 255.f;
	float Distance1 = EtcParameters.RGB_DISTANCE_TABLES[TableIdx].w / 255.f;
	
	float BaseLum = EtcLuminance(BaseColor);
	float Lum1 = min(1.f, BaseLum + Distance1);
	float Lum0 = BaseLum + Distance0;
	float Lum3 = max(0.f, BaseLum - Distance1);
	float Lum2 = BaseLum - Distance0;
	
	for (uint Y = 0; Y < 4; ++Y)
	{
		for (uint X = 0; X < 4; ++X)
		{
			float3 Color = Block[4u * Y + X];
			float ColorLum = EtcLuminance(Color);
			uint EncIndex = 0u;
			if (ColorLum < BaseLum || ColorLum == 0.f) // make sure fully Black block is encoded correctly
			{
				EncIndex = (ColorLum - Lum3) < (Lum2 - ColorLum) ? 3u : 2u;
			}
			else
			{
				EncIndex = (Lum1 - ColorLum) < (ColorLum - Lum0) ? 1u : 0u;
			}
			
			uint IndexInBlock = X * 4u + Y;
			BlockWeights |= ((EncIndex & 1u) << IndexInBlock) | ((EncIndex >> 1u) << (16u + IndexInBlock));
		}
	}
	return BlockWeights;
}

uint EtcSelectRGBTableIndex(float LuminanceR)
{
	// guess a table using sub-block luminance range
	float Range = LuminanceR*255.f + 1.f;
	if (Range < 8.0)
	{
		return 0u;
	}
	else if (Range < 17.0)
	{
		return 1u;
	}
	else if (Range < 29.0)
	{
		return 2u;
	}
	else if (Range < 42.0)
	{
		return 3u;
	}
	else if (Range < 60.0)
	{
		return 4u;
	}
	else if (Range < 80.0)
	{
		return 5u;
	}
	else if (Range < 106.0)
	{
		return 6u;
	}
	return 7u;
}

uint2 CompressBlock_ETC1S(in float3 Block[16], bool bSRGB, bool bCoCg)
{
	// ETC1S: differential with color delta=0
	uint FlipBit = 0u;
	uint BlockModeBits = (BLOCK_MODE_DIFFERENTIAL << 25u);

	float3 BaseColorMin = 1;
	float3 BaseColorMax = 0;
	float3 BaseColorAvg = 0;
	for (int i = 0; i < 16; ++i)
	{
		BaseColorMin = min(BaseColorMin, Block[i]);
		BaseColorMax = max(BaseColorMax, Block[i]);
		BaseColorAvg = BaseColorAvg + Block[i];
	}
	BaseColorAvg = BaseColorAvg / 16.f;
		
	float3 BaseColor = (bSRGB ? LinearToSrgb(BaseColorAvg) : BaseColorAvg);
	
	uint ColorBits = 0u;
	if (bCoCg)
	{
		float CoCgScale = EtcGetYCoCgScale(BaseColorMin.xy, BaseColorMax.xy);
		EtcApplyYCoCgScale(BaseColor.xy, CoCgScale);
		ColorBits = EtcPackColorBits(BaseColor);
		ColorBits |= (((uint)CoCgScale - 1) << 19u);
	}
	else
	{
		ColorBits = EtcPackColorBits(BaseColor);
	}

	float LumMin = EtcLuminance(BaseColorMin);
	float LumMax = EtcLuminance(BaseColorMax);
	float LumRange = (LumMax - LumMin) * 0.5f;

	uint BlockTableIdx = EtcSelectRGBTableIndex(LumRange);
	uint BlockWeights = EtcGetBlockWeights(Block, BaseColorAvg, BlockTableIdx);

	// Both these values need to be big-endian. We can build ModeBits directly in big-endian layout, but for IndexBits
	// it's too hard, so we'll just swap here.
	uint ModeBits = (BlockTableIdx << 29u) | (BlockTableIdx << 26u) | (BlockModeBits) | (FlipBit << 24u) | ColorBits;
	uint IndexBits = EtcSwapEndian32(BlockWeights);

	return uint2(ModeBits, IndexBits);
}

uint2 CompressBlock_ETC2_SRGB(in float3 Block[16])
{
	const bool bSRGB = true;
	const bool bCoCg = false;
	return CompressBlock_ETC1S(Block, bSRGB, bCoCg);
}

uint2 CompressBlock_ETC2_RGB(in float3 Block[16])
{
	const bool bSRGB = false;
	const bool bCoCg = false;
	return CompressBlock_ETC1S(Block, bSRGB, bCoCg);
}

void SelectAlphaMod(in float SourceAlpha, in float EncodedAlpha, int IndexInTable, inout int SelectedIndex, inout float MinDif)
{
	float Dif = abs(EncodedAlpha - SourceAlpha);
	if (Dif < MinDif)
	{
		MinDif = Dif;
		SelectedIndex = IndexInTable;
	}
}

uint2 CompressBlock_ETC2_Alpha(in float BlockA[16])
{
	float MinAlpha = 1.f;
	float MaxAlpha = 0.f;
	for (int k = 0; k < 16; ++k)
	{
		float A = BlockA[k];
		MinAlpha = min(A, MinAlpha);
		MaxAlpha = max(A, MaxAlpha);
	}

	MinAlpha = round(MinAlpha*255.f);
	MaxAlpha = round(MaxAlpha*255.f);
	
	float AlphaRange = MaxAlpha - MinAlpha;
	const float MidRange = 21.f;// an average range in ALPHA_DISTANCE_TABLES
	float Multiplier = clamp(round(AlphaRange/MidRange), 1.f, 15.f);
	
	int TableIdx = 0;
	float4 TableValueNeg = float4(0,0,0,0);
	float4 TableValuePos = float4(0,0,0,0);
	
	// iterating through all tables to find a best fit is quite slow
	// instead guess the best table based on alpha range
	const int TablesToTest[5] = {15,11,6,2,0};
	for (int i = 0; i < 5; ++i)
	{
		TableIdx = TablesToTest[i];
		TableValuePos = EtcParameters.ALPHA_DISTANCE_TABLES[TableIdx];
				
		float TableRange = (TableValuePos.w*2 + 1)*Multiplier;
		float Dif = TableRange - AlphaRange;
		if (Dif > 0.f)
		{
			break;
		}
	}
	TableValueNeg = -(TableValuePos + float4(1,1,1,1));
	
	TableValuePos*=Multiplier;
	TableValueNeg*=Multiplier;
	
	// make sure an exact value of MinAlpha can always be decoded from a BaseValue
	float BaseValue = clamp(round(-TableValueNeg.w + MinAlpha), 0.f, 255.f);
	
	TableValueNeg = TableValueNeg + BaseValue.xxxx;
	TableValuePos = TableValuePos + BaseValue.xxxx;
	uint2 BlockWeights = 0;
	
	for (int PixelIndex = 0; PixelIndex < 16; ++PixelIndex)
	{
		float Alpha = BlockA[PixelIndex]*255.f;
		int SelectedIndex = 0;
		float MinDif = 100000.f;
		
		if (Alpha < TableValuePos.x)
		{
			SelectAlphaMod(Alpha, TableValueNeg.x, 0, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValueNeg.y, 1, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValueNeg.z, 2, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValueNeg.w, 3, SelectedIndex, MinDif);
		}
		else
		{
			SelectAlphaMod(Alpha, TableValuePos.x, 4, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValuePos.y, 5, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValuePos.z, 6, SelectedIndex, MinDif);
			SelectAlphaMod(Alpha, TableValuePos.w, 7, SelectedIndex, MinDif);
		}

		// ETC uses column-major indexing for the pixels in a block...
		int TransposedIndex = (PixelIndex >> 2) | ((PixelIndex & 3) << 2);
		int StartBit = (15 - TransposedIndex) * 3;
		BlockWeights.x |= (StartBit < 32) ? SelectedIndex << StartBit : 0;
		int ShiftRight = (StartBit == 30) ? 2 : 0;
		int ShiftLeft = (StartBit >= 32) ? StartBit - 32 : 0;
		BlockWeights.y |= (StartBit >= 30) ? (SelectedIndex >> ShiftRight) << ShiftLeft : 0;
	}

	int MultiplierInt = Multiplier;
	int BaseValueInt = BaseValue;
	
	uint2 AlphaBits;
	AlphaBits.x = EtcSwapEndian32(BlockWeights.y | (TableIdx << 16) | (MultiplierInt << 20) | (BaseValueInt << 24));
	AlphaBits.y = EtcSwapEndian32(BlockWeights.x);

	return AlphaBits;
}

uint4 CompressBlock_ETC2_RGBA(in float3 BlockRGB[16], in float BlockA[16])
{
	uint2 CompressedRGB = CompressBlock_ETC2_RGB(BlockRGB);
	uint2 CompressedAlpha = CompressBlock_ETC2_Alpha(BlockA);
	return uint4(CompressedAlpha, CompressedRGB);
}

uint4 CompressBlock_ETC2_SRGBA(in float3 BlockRGB[16], in float BlockA[16])
{
	uint2 CompressedRGB = CompressBlock_ETC2_SRGB(BlockRGB);
	uint2 CompressedAlpha = CompressBlock_ETC2_Alpha(BlockA);
	return uint4(CompressedAlpha, CompressedRGB);
}

uint4 CompressBlock_ETC2_YCoCg(in float3 BlockRGB[16])
{
	float BlockY[16];
	for (int i = 0; i < 16; ++i)
	{
		float3 YCoCg = RGB2YCoCg(BlockRGB[i]);
		BlockRGB[i] = float3(YCoCg.yz, 0.0);
		BlockY[i] = YCoCg.x;
	}

	const bool bSRGB = false;
	const bool bCoCg = true;
	uint2 CompressedRGB = CompressBlock_ETC1S(BlockRGB, bSRGB, bCoCg);
	uint2 CompressedAlpha = CompressBlock_ETC2_Alpha(BlockY);
	return uint4(CompressedAlpha, CompressedRGB);
}

uint4 CompressBlock_ETC2_RG(in float BlockU[16], in float BlockV[16])
{
	uint2 R = CompressBlock_ETC2_Alpha(BlockU);
	uint2 G = CompressBlock_ETC2_Alpha(BlockV);
	return uint4(R, G);
}
