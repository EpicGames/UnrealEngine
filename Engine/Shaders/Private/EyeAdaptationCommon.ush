// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

#if !EYE_ADAPTATION_DISABLED && !EYE_ADAPTATION_PREV_FRAME_EXPOSURE
StructuredBuffer<float4> EyeAdaptationBuffer;

float4 EyeAdaptationLookupBuffer(StructuredBuffer<float4> InEyeAdaptation)
{
	return InEyeAdaptation[0];
}
#endif // !EYE_ADAPTATION_DISABLED && !EYE_ADAPTATION_PREV_FRAME_EXPOSURE

#if defined (EyeAdaptationStruct)
	#if EYE_ADAPTATION_LOOSE_PARAMETERS
		#error EyeAdaptationStruct and EYE_ADAPTATION_LOOSE_PARAMETERS cannot both be defined.
	#endif
#endif

// This eye adaptation lookup function is used by the material template. However, post process materials also require
// it, but are themselves global shaders which can utilize loose parameters instead. Rules for the implementation:
//  - Defining EYE_ADAPTATION_DISABLED will cause it to always return 0.
//  - Defining EyeAdaptationStruct will cause it to pull from that uniform buffer struct.
//  - Defining EYE_ADAPTATION_LOOSE_PARAMETERS will cause it to pull from the loose texture / buffer.
//  - Doing none of the above will fall back to disabled.

float EyeAdaptationLookup()
{
#if EYE_ADAPTATION_DISABLED
	return 0.0f;
#elif defined (EyeAdaptationStruct)
	#if SHADING_PATH_DEFERRED
		return EyeAdaptationLookupBuffer(EyeAdaptationStruct.EyeAdaptationBuffer).x;
	#elif SHADING_PATH_MOBILE
		// Unfortunately on older MALI GL drivers (r9p0), passing this buffer into 'log' or other functions, fails with this error: Function call discards 'readonly' access qualifier
		// So we have to use last frame eye adaptation value as a workaround on mobile
		// return EyeAdaptationLookupBuffer(EyeAdaptationBuffer).x;
		return ResolvedView.PreExposure;
	#else
		#error Unknown shading path.
	#endif
#elif EYE_ADAPTATION_LOOSE_PARAMETERS
	return EyeAdaptationLookupBuffer(EyeAdaptationBuffer).x;
#elif EYE_ADAPTATION_PREV_FRAME_EXPOSURE
	// fallback implementation if the EyeAdaptationBuffer is not available
	return ResolvedView.PreExposure;
#else
	return 0.0f;
#endif
}

float InverseExposureLerp(float Exposure, float Alpha)
{
#if EYE_ADAPTATION_DISABLED
	return 1.0f;
#else
	// When Alpha=0.0, we want to multiply by 1.0. when Alpha = 1.0, we want to multiply by 1/Exposure.
	// So the lerped value is:
	//     LerpLogScale = Lerp(log(1),log(1/Exposure),T)
	// Which is simplified as:
	//     LerpLogScale = Lerp(0,-log(Exposure),T)
	//     LerpLogScale = -T * log(Exposure);

	float LerpLogScale = -Alpha * log(Exposure);
	float Scale = exp(LerpLogScale);
	return Scale;
#endif
}

float EyeAdaptationInverseLookup(float Alpha)
{
	return InverseExposureLerp(EyeAdaptationLookup(), Alpha);
}

float3 EyeAdaptationInverseLookup(float3 LightValue, float Alpha)
{
	return LightValue * EyeAdaptationInverseLookup(Alpha);
}