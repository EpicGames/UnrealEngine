// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

FScreenTransform SvPositionToInputTextureUV;

Texture2D Input_Texture;
SamplerState Input_Sampler;

// FXAA_PRESET is set on C++ side
// 0:low quality but fast, .. 5: high quality but slow

// depending on the FXAA preset (formerly from 0 to 5
// we chose the FXAA settings
#if FXAA_PRESET == 0
	#define FXAA_QUALITY__PRESET 10
	#define FXAA_PC_CONSOLE 1
#elif FXAA_PRESET == 1
	#define FXAA_QUALITY__PRESET 10
	#define FXAA_PC 1
#elif FXAA_PRESET == 2
	#define FXAA_QUALITY__PRESET 13
	#define FXAA_PC 1
#elif FXAA_PRESET == 3
	#define FXAA_QUALITY__PRESET 15
	#define FXAA_PC 1
#elif FXAA_PRESET == 4
	#define FXAA_QUALITY__PRESET 29
	#define FXAA_PC 1
#elif FXAA_PRESET == 5
	#define FXAA_QUALITY__PRESET 39
	#define FXAA_PC 1
#endif

// make sure FXAA_360_OPT is 0 or 1
#ifndef FXAA_360_OPT
	#define FXAA_360_OPT 0
#endif


/*-----------------------------------------------------------------------------
	FXAA pixel shader
-----------------------------------------------------------------------------*/
#if PIXELSHADER

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	#define FXAA_HLSL_5 1
#else
	#define FXAA_HLSL_4 1
#endif
	
	#define DECLARE_PARAMETER(a, b, c)

	// see Fxaa.. header file for more details
	float4 fxaaConsoleRcpFrameOpt;
	float4 fxaaConsoleRcpFrameOpt2;
	float4 fxaaConsole360RcpFrameOpt2;
	float fxaaQualitySubpix;
	float fxaaQualityEdgeThreshold;
	float fxaaQualityEdgeThresholdMin;
	float fxaaConsoleEdgeSharpness;
	float fxaaConsoleEdgeThreshold;
	float fxaaConsoleEdgeThresholdMin;
	float4 fxaaConsole360ConstDir;

	// include NVIDIA FXAA source
	#include "Fxaa3_11.ush"

	void FxaaPS(
		float4 SvPosition : SV_POSITION,
		out float4 OutColor : SV_Target0)
	{
		FxaaTex TextureAndSampler;
		TextureAndSampler.tex = Input_Texture;
		TextureAndSampler.smpl = Input_Sampler;
		TextureAndSampler.UVMinMax = float4(Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);
		
		float2 TexCenter = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
		float4 TexCorners = float4(TexCenter - 0.5f * Input_ExtentInverse, TexCenter + 0.5f * Input_ExtentInverse);

		OutColor = FxaaPixelShader(
			TexCenter, TexCorners,
			TextureAndSampler,
			TextureAndSampler,
			TextureAndSampler,
			Input_ExtentInverse,
			fxaaConsoleRcpFrameOpt,
			fxaaConsoleRcpFrameOpt2,
			fxaaConsole360RcpFrameOpt2,
			fxaaQualitySubpix,
			fxaaQualityEdgeThreshold,
			fxaaQualityEdgeThresholdMin,
			fxaaConsoleEdgeSharpness,
			fxaaConsoleEdgeThreshold,
			fxaaConsoleEdgeThresholdMin,
			fxaaConsole360ConstDir);

		#if DIM_ALPHA_CHANNEL == 0
			OutColor.a = 1.0;
		#endif
	}

#endif // PIXELSHADER
