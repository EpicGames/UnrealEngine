// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FastMath.ush: Fast/approximated math functions
=============================================================================*/
  
#pragma once

#include "FastMathThirdParty.ush"

// Reference: http://www.humus.name/Articles/Persson_LowLevelThinking.pdf p.27
#define FastExp_N(T) T FastExp(T x) { return exp2(1.442695f * x); }
FastExp_N(float)
FastExp_N(float2)
FastExp_N(float3)
FastExp_N(float4)

// Reference: http://www.humus.name/Articles/Persson_LowLevelThinking.pdf p.27
// Corrected version. The factor is incorrectly placed in the reference.
#define FastLog_N(T) T FastLog(T x) { return 0.693147f*log2(x); }
FastLog_N(float)
FastLog_N(float2)
FastLog_N(float3)
FastLog_N(float4)