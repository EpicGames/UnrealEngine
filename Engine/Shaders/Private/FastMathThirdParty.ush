// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

/******************************************************************************
    Shader Fast Math Lib (v0.41)
    A shader math library for optimized approximate transcendental functions.
    Optimized and tested on AMD GCN architecture.
********************************************************************************/

/******************************************************************************
    The MIT License (MIT)
    Copyright (c) <2014> <Michal Drobot>
    Permission is hereby granted, free of charge, to any person obtaining a copy
    of this software and associated documentation files (the "Software"), to deal
    in the Software without restriction, including without limitation the rights
    to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
    copies of the Software, and to permit persons to whom the Software is
    furnished to do so, subject to the following conditions:
    The above copyright notice and this permission notice shall be included in
    all copies or substantial portions of the Software.
    THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
    IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
    FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
    AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
    LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
    OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN
    THE SOFTWARE.
********************************************************************************/

//
// Normalized range [0,1] Constants
//
#define IEEE_INT_RCP_CONST_NR0_SNORM        0x7EEF370B
#define IEEE_INT_SQRT_CONST_NR0_SNORM       0x1FBD1DF5
#define IEEE_INT_RCP_SQRT_CONST_NR0_SNORM   0x5F341A43


// Relative error : ~3.4% over full
// Precise format : ~small float
// 2 ALU
float rsqrtFast( float x )
{
	int i = asint(x);
	i = 0x5f3759df - (i >> 1);
	return asfloat(i);
}

// Relative error : < 0.7% over full
// Precise format : ~small float
// 1 ALU
float sqrtFast( float x )
{
	int i = asint(x);
	i = 0x1FBD1DF5 + (i >> 1);
	return asfloat(i);
}

// Relative error : < 0.4% over full
// Precise format : ~small float
// 1 ALU
float rcpFast( float x )
{
	int i = asint(x);
	i = 0x7EF311C2 - i;
	return asfloat(i);
}

// Using 1 Newton Raphson iterations
// Relative error : < 0.02% over full
// Precise format : ~half float
// 3 ALU
float rcpFastNR1( float x )
{
	int i = asint(x);
	i = 0x7EF311C3 - i;
	float xRcp = asfloat(i);
	xRcp = xRcp * (-xRcp * x + 2.0f);
	return xRcp;
}

float lengthFast( float3 v )
{
	float LengthSqr = dot(v,v);
	return sqrtFast( LengthSqr );
}

float3 normalizeFast( float3 v )
{
	float LengthSqr = dot(v,v);
	return v * rsqrtFast( LengthSqr );
}

#if COMPILER_SUPPORTS_MED3
	#define fastClamp(x, Min, Max) med3(x, Min, Max)
#else
	#define fastClamp(x, Min, Max) clamp(x, Min, Max)
#endif

//
// Trigonometric functions
//

// max absolute error 9.0x10^-3
// Eberly's polynomial degree 1 - respect bounds
// 4 VGPR, 12 FR (8 FR, 1 QR), 1 scalar
// input [-1, 1] and output [0, PI]
float acosFast(float inX) 
{
    float x = abs(inX);
    float res = -0.156583f * x + (0.5 * PI);
    res *= sqrt(1.0f - x);
    return (inX >= 0) ? res : PI - res;
}

float2 acosFast( float2 x )
{
	return float2( acosFast(x.x), acosFast(x.y) );
}

float3 acosFast( float3 x )
{
	return float3( acosFast(x.x), acosFast(x.y), acosFast(x.z) );
}

float4 acosFast( float4 x )
{
	return float4( acosFast(x.x), acosFast(x.y), acosFast(x.z), acosFast(x.w) );
}

// Same cost as acosFast + 1 FR
// Same error
// input [-1, 1] and output [-PI/2, PI/2]
float asinFast( float x )
{
    return (0.5 * PI) - acosFast(x);
}

float2 asinFast( float2 x)
{
	return float2( asinFast(x.x), asinFast(x.y) );
}

float3 asinFast( float3 x)
{
	return float3( asinFast(x.x), asinFast(x.y), asinFast(x.z) );
}

float4 asinFast( float4 x )
{
	return float4( asinFast(x.x), asinFast(x.y), asinFast(x.z), asinFast(x.w) );
}

// max absolute error 1.3x10^-3
// Eberly's odd polynomial degree 5 - respect bounds
// 4 VGPR, 14 FR (10 FR, 1 QR), 2 scalar
// input [0, infinity] and output [0, PI/2]
float atanFastPos( float x ) 
{ 
    float t0 = (x < 1.0f) ? x : 1.0f / x;
    float t1 = t0 * t0;
    float poly = 0.0872929f;
    poly = -0.301895f + poly * t1;
    poly = 1.0f + poly * t1;
    poly = poly * t0;
    return (x < 1.0f) ? poly : (0.5 * PI) - poly;
}

// 4 VGPR, 16 FR (12 FR, 1 QR), 2 scalar
// input [-infinity, infinity] and output [-PI/2, PI/2]
float atanFast( float x )
{
    float t0 = atanFastPos( abs(x) );
    return (x < 0) ? -t0: t0;
}

float2 atanFast( float2 x )
{
	return float2( atanFast(x.x), atanFast(x.y) );
}

float3 atanFast( float3 x )
{
	return float3( atanFast(x.x), atanFast(x.y), atanFast(x.z) );
}

float4 atanFast( float4 x )
{
	return float4( atanFast(x.x), atanFast(x.y), atanFast(x.z), atanFast(x.w) );
}

float atan2Fast( float y, float x )
{
	float t0 = max( abs(x), abs(y) );
	float t1 = min( abs(x), abs(y) );
	float t3 = t1 / t0;
	float t4 = t3 * t3;

	// Same polynomial as atanFastPos
	t0 =         + 0.0872929;
	t0 = t0 * t4 - 0.301895;
	t0 = t0 * t4 + 1.0;
	t3 = t0 * t3;

	t3 = abs(y) > abs(x) ? (0.5 * PI) - t3 : t3;
	t3 = x < 0 ? PI - t3 : t3;
	t3 = y < 0 ? -t3 : t3;

	return t3;
}

float2 atan2Fast( float2 y, float2 x )
{
	return float2( atan2Fast(y.x, x.x), atan2Fast(y.y, x.y) );
}

float3 atan2Fast( float3 y, float3 x )
{
	return float3( atan2Fast(y.x, x.x), atan2Fast(y.y, x.y), atan2Fast(y.z, x.z) );
}

float4 atan2Fast( float4 y, float4 x )
{
	return float4( atan2Fast(y.x, x.x), atan2Fast(y.y, x.y), atan2Fast(y.z, x.z), atan2Fast(y.w, x.w) );
}

// 4th order polynomial approximation
// 4 VGRP, 16 ALU Full Rate
// 7 * 10^-5 radians precision
// Reference : Handbook of Mathematical Functions (chapter : Elementary Transcendental Functions), M. Abramowitz and I.A. Stegun, Ed.
float acosFast4(float inX)
{
	float x1 = abs(inX);
	float x2 = x1 * x1;
	float x3 = x2 * x1;
	float s;

	s = -0.2121144f * x1 + 1.5707288f;
	s = 0.0742610f * x2 + s;
	s = -0.0187293f * x3 + s;
	s = sqrt(1.0f - x1) * s;

	// acos function mirroring
	// check per platform if compiles to a selector - no branch neeeded
	return inX >= 0.0f ? s : PI - s;
}

// 4th order polynomial approximation
// 4 VGRP, 16 ALU Full Rate
// 7 * 10^-5 radians precision 
float asinFast4( float x )
{
	return (0.5 * PI) - acosFast4(x);
}

// @param A doesn't have to be normalized, output could be NaN if this is near 0,0,0
// @param B doesn't have to be normalized, output could be NaN if this is near 0,0,0
// @return can be passed to a acosFast() or acos() to compute an angle
float CosBetweenVectors(float3 A, float3 B)
{
	// unoptimized: dot(normalize(A), normalize(B))
	return dot(A, B) * rsqrt(length2(A) * length2(B));
}

// @param A doesn't have to be normalized, output could be NaN if this is near 0,0,0
// @param B doesn't have to be normalized, output could be NaN if this is near 0,0,0
float AngleBetweenVectors(float3 A, float3 B)
{
	return acos(CosBetweenVectors(A, B));
}
// @param A doesn't have to be normalized, output could be NaN if this is near 0,0,0
// @param B doesn't have to be normalized, output could be NaN if this is near 0,0,0
float AngleBetweenVectorsFast(float3 A, float3 B)
{
	return acosFast(CosBetweenVectors(A, B));
}

// Returns sign bit of floating point as either 1 or -1.
int SignFastInt(float v)
{
	return 1 - int((asuint(v) & 0x80000000) >> 30);
}

int2 SignFastInt(float2 v)
{
	return int2(SignFastInt(v.x), SignFastInt(v.y));
}
