// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FilterPixelShader.usf: Filter pixel shader source.
=============================================================================*/

#define SCENE_TEXTURES_DISABLED 1

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"

#ifndef STATIC_SAMPLE_COUNT
	#error STATIC_SAMPLE_COUNT is undefined
#endif

#ifndef SAMPLE_COUNT_MAX
	#error SAMPLE_COUNT_MAX is undefined
#endif

#if STATIC_SAMPLE_COUNT > SAMPLE_COUNT_MAX
#error STATIC_SAMPLE_COUNT cannot be greater than SAMPLE_COUNT_MAX
#endif

#define PACKED_STATIC_SAMPLE_COUNT ((STATIC_SAMPLE_COUNT + 1) / 2)
#define PACKED_SAMPLE_COUNT_MAX ((SAMPLE_COUNT_MAX + 1) / 2)

#define USE_UNPACKED_UVS (ES3_1_PROFILE)
#define USE_DYNAMIC_LOOP (STATIC_SAMPLE_COUNT == 0)

SCREEN_PASS_TEXTURE_VIEWPORT(Filter)

SamplerState Filter_Sampler;
Texture2D Filter_Texture; 

float4 SampleFilterTexture(float2 UV)
{
#if USE_MANUAL_UV_BORDER
	float2 ClampedUV = clamp(UV, Filter_UVViewportBilinearMin, Filter_UVViewportBilinearMax);

	float4 Sample = Texture2DSampleLevel(Filter_Texture, Filter_Sampler, ClampedUV, 0);

	float2 TexelOffset = abs(ClampedUV - UV) * Filter_Extent;

	float2 BilinearWeight = saturate(1.0f - TexelOffset);

	return BilinearWeight.x * BilinearWeight.y * Sample;
#else
	return Texture2DSampleLevel(Filter_Texture, Filter_Sampler, UV, 0);
#endif
}

#if USE_COMBINE_ADDITIVE

SCREEN_PASS_TEXTURE_VIEWPORT(Additive)
SamplerState Additive_Sampler;
Texture2D Additive_Texture;

float4 SampleAdditiveTexture(float2 UV)
{
#if USE_MANUAL_UV_BORDER
	UV = clamp(UV, Additive_UVViewportBilinearMin, Additive_UVViewportBilinearMax);
#endif

	return Texture2DSampleLevel(Additive_Texture, Additive_Sampler, UV, 0);
}

#endif

#if USE_DYNAMIC_LOOP

FScreenTransform SvPositionToTextureUV;

float4 SampleOffsets[PACKED_SAMPLE_COUNT_MAX];
float4 SampleWeights[SAMPLE_COUNT_MAX];
int SampleCount;

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 InUV = ApplyScreenTransform(SvPosition.xy, SvPositionToTextureUV);

	float4 Color = 0;

	int SampleIndex = 0;

	LOOP for (; SampleIndex < SampleCount - 1; SampleIndex += 2)
	{
		float4 UVUV = InUV.xyxy + SampleOffsets[SampleIndex / 2];
		Color += SampleFilterTexture(UVUV.xy) * SampleWeights[SampleIndex + 0];
		Color += SampleFilterTexture(UVUV.zw) * SampleWeights[SampleIndex + 1];
	}

	BRANCH if (SampleIndex < SampleCount)
	{
		float2 UV = InUV + SampleOffsets[SampleIndex / 2].xy;
		Color += SampleFilterTexture(UV) * SampleWeights[SampleIndex + 0];
	}

#if USE_COMBINE_ADDITIVE
	Color += SampleAdditiveTexture(InUV);
#endif

	OutColor = Color;
}

#else // !USE_DYNAMIC_LOOP

float4 SampleOffsets[PACKED_STATIC_SAMPLE_COUNT];
float4 SampleWeights[STATIC_SAMPLE_COUNT];

void MainPS(
	noperspective float2 InUV : TEXCOORD0,
#if USE_UNPACKED_UVS
	noperspective float2 InOffsetUVs[STATIC_SAMPLE_COUNT] : TEXCOORD1,
#else
	noperspective float4 InOffsetUVs[PACKED_STATIC_SAMPLE_COUNT] : TEXCOORD1,
#endif
	out MaterialFloat4 OutColor : SV_Target0)
{
	float4 Color = 0;

#if USE_UNPACKED_UVS
	UNROLL for (int i = 0; i < STATIC_SAMPLE_COUNT; ++i)
	{
		Color += SampleFilterTexture(InOffsetUVs[i]) * SampleWeights[i];
	}
#else
	UNROLL for (int SampleIndex = 0; SampleIndex < STATIC_SAMPLE_COUNT - 1; SampleIndex += 2)
	{
		float4 UVUV = InOffsetUVs[SampleIndex / 2];
		Color += SampleFilterTexture(UVUV.xy) * SampleWeights[SampleIndex + 0];
		Color += SampleFilterTexture(UVUV.zw) * SampleWeights[SampleIndex + 1];
	}

	FLATTEN if (STATIC_SAMPLE_COUNT & 1)
	{
		float2 UV = InOffsetUVs[PACKED_STATIC_SAMPLE_COUNT - 1].xy;
		Color += SampleFilterTexture(UV) * SampleWeights[STATIC_SAMPLE_COUNT - 1];
	}
#endif

#if USE_COMBINE_ADDITIVE
	Color += SampleAdditiveTexture(InUV);
#endif

	OutColor = Color;
}

#endif

#if COMPUTESHADER

SCREEN_PASS_TEXTURE_VIEWPORT(Output)

RWTexture2D<float4> RWOutputTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 iPixelPos = DispatchThreadId + Output_ViewportMin;
	float2 PixelPos = float2(iPixelPos);
	float2 BaseUV = (PixelPos + 0.5) * Output_ExtentInverse;

	if (IsComputeUVOutOfBounds(BaseUV))
	{
		return;
	}

	float4 Color = 0;
	int SampleIndex = 0;

#if USE_DYNAMIC_LOOP
	LOOP for (; SampleIndex < SampleCount - 1; SampleIndex += 2)
#else
	UNROLL for (; SampleIndex < STATIC_SAMPLE_COUNT - 1; SampleIndex += 2)
#endif
	{
		float4 UVUV = BaseUV.xyxy + SampleOffsets[SampleIndex / 2];
		Color += SampleFilterTexture(UVUV.xy) * SampleWeights[SampleIndex + 0];
		Color += SampleFilterTexture(UVUV.zw) * SampleWeights[SampleIndex + 1];
	}

#if USE_DYNAMIC_LOOP
	BRANCH if (SampleIndex < SampleCount)
#else
	FLATTEN if (SampleIndex < STATIC_SAMPLE_COUNT)
#endif
	{
		float2 UV = BaseUV + SampleOffsets[SampleIndex / 2].xy;

		Color += SampleFilterTexture(UV) * SampleWeights[SampleIndex + 0];
	}

#if USE_COMBINE_ADDITIVE
	Color += SampleAdditiveTexture(BaseUV);
#endif

	RWOutputTexture[PixelPos] = Color;
}

#endif 