// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FilterVertexShader.usf: Filter vertex shader source.
=============================================================================*/

#include "Common.ush"
#include "ScreenPass.ush"

#ifndef STATIC_SAMPLE_COUNT
	#error Static sample count is undefined!
#endif

#define PACKED_STATIC_SAMPLE_COUNT ((STATIC_SAMPLE_COUNT + 1) / 2)

#define USE_UNPACKED_UVS (ES3_1_PROFILE)

FScreenTransform ViewportUVToTextureUV;

float4 SampleOffsets[PACKED_STATIC_SAMPLE_COUNT];

#if USE_UNPACKED_UVS

void MainVS(
	float2 ViewportUV : ATTRIBUTE0,
	float2 Unused     : ATTRIBUTE1, // TODO: kill
	out noperspective float2 OutUV : TEXCOORD0,
	out noperspective float2 OutOffsetUVs[STATIC_SAMPLE_COUNT] : TEXCOORD1,
	out float4 OutPosition : SV_POSITION)
{
	OutPosition = float4(ViewportUVToScreenPos(ViewportUV), 0, 1);
	
	float2 UV = ApplyScreenTransform(ViewportUV, ViewportUVToTextureUV);
	
	for (int SampleIndex = 0; SampleIndex < STATIC_SAMPLE_COUNT - 1; SampleIndex += 2)
	{
		half4 OffsetUVUV = SampleOffsets[SampleIndex / 2];
		OutOffsetUVs[SampleIndex + 0] = UV.xy + OffsetUVUV.xy;
		OutOffsetUVs[SampleIndex + 1] = UV.xy + OffsetUVUV.zw;
	}

	if (STATIC_SAMPLE_COUNT & 1)
	{
		half4 OffsetUVUV = SampleOffsets[PACKED_STATIC_SAMPLE_COUNT - 1];
		OutOffsetUVs[STATIC_SAMPLE_COUNT - 1] = UV.xy + OffsetUVUV.xy;
	}

	OutUV = UV;
}

#else

void MainVS(
	float2 ViewportUV : ATTRIBUTE0,
	float2 Unused     : ATTRIBUTE1, // TODO: kill
	out noperspective float2 OutUV : TEXCOORD0,
	out noperspective float4 OutOffsetUVs[PACKED_STATIC_SAMPLE_COUNT] : TEXCOORD1,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition = float4(ViewportUVToScreenPos(ViewportUV), 0, 1);
	
	float2 UV = ApplyScreenTransform(ViewportUV, ViewportUVToTextureUV);
	
	UNROLL for(int OffsetIndex = 0; OffsetIndex < PACKED_STATIC_SAMPLE_COUNT; ++OffsetIndex)
	{
		OutOffsetUVs[OffsetIndex] = UV.xyxy + SampleOffsets[OffsetIndex];
	}

	OutUV = UV;
}

#endif