// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "Substrate/Substrate.ush"

#define FP_MAX_VALID_HIT_DISTANCE (254.0f / 255.0f)

bool IsFirstPersonPixel(uint2 PixelPos)
{
#if SUBSTRATE_GBUFFER_FORMAT
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bIsFirstPerson = SubstratePixelHeader.IsFirstPerson();
#else
	const float2 BufferUV = (PixelPos + 0.5f) * View.BufferSizeAndInvSize.zw;
	const FGBufferData GBuffer = GetGBufferData(BufferUV);
	const bool bIsFirstPerson = IsFirstPerson(GBuffer);
#endif // SUBSTRATE_ENABLED

	return bIsFirstPerson;
}

float3 GetWorldNormal(uint2 PixelPos)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(int3(PixelPos, 0)));
	const float3 WorldNormal = TopLayerData.WorldNormal;
#else
	// Normals are in a different texture than the first person bit, so we do not redundantly sample the same texture twice (assuming the compiler strips the unused GBuffer values).
	const float2 BufferUV = (PixelPos + 0.5f) * View.BufferSizeAndInvSize.zw;
	const FGBufferData GBuffer = GetGBufferData(BufferUV);
	const float3 WorldNormal = GBuffer.WorldNormal;
#endif

	return WorldNormal;
}

#ifdef FirstPersonSelfShadowDownsamplePS

uint CheckerboardMode; // 0: always min, 1: always max, 2: checkerboard

void FirstPersonSelfShadowDownsamplePS(
	in float4 SVPos : SV_Position,
	out float4 OutColor : SV_Target0,
	out float OutDepth : SV_Depth
)
{
	const uint2 DstPixelPos = uint2(SVPos.xy);
	float MinZ = FarDepthValue;
	float MaxZ = FarDepthValue;
	uint2 MinZPixelPos = 0u;
	uint2 MaxZPixelPos = 0u;
	for (uint y = 0; y < 2; ++y)
	{
		for (uint x = 0; x < 2; ++x)
		{
			const uint2 FullResPixelPos = min(DstPixelPos * 2u + uint2(x, y), View.ViewRectMinAndSize.zw) + View.ViewRectMinAndSize.xy;
			float DeviceZ = LookupDeviceZ(FullResPixelPos);

			// Ignore far depth values
			if (DeviceZ != FarDepthValue)
			{
				const bool bIsFirstPerson = IsFirstPersonPixel(FullResPixelPos);

				// When the pixel is not first person, set DeviceZ to FarDepthValue so that we ignore it in the checks below.
				DeviceZ = bIsFirstPerson ? DeviceZ : FarDepthValue;

				if (DeviceZ != FarDepthValue && (MinZ == FarDepthValue || DeviceZ < MinZ))
				{
					MinZ = DeviceZ;
					MinZPixelPos = FullResPixelPos;
				}
				if (DeviceZ != FarDepthValue && (MaxZ == FarDepthValue || DeviceZ > MaxZ))
				{
					MaxZ = DeviceZ;
					MaxZPixelPos = FullResPixelPos;
				}
			}
		}
	}

	bool bCheckerboard = false;
	switch (CheckerboardMode)
	{
	case 0: bCheckerboard = true; break;
	case 1: bCheckerboard = false; break;
	case 2: bCheckerboard = ((DstPixelPos.x + DstPixelPos.y) & 1u) != 0u; break;
	}
	const float DownsampledDepth = bCheckerboard ? MinZ : MaxZ;

	// Early out
	if (DownsampledDepth == FarDepthValue)
	{
		discard;
	}

	const uint2 FullResPixelPos = bCheckerboard ? MinZPixelPos : MaxZPixelPos;
	const float3 WorldNormal = GetWorldNormal(FullResPixelPos);
	const float2 WorldNormalOct = UnitVectorToOctahedron(WorldNormal);

	const uint2 BaseFullResPixelPos = min(DstPixelPos * 2u, View.ViewRectMinAndSize.zw) + View.ViewRectMinAndSize.xy;
	const uint2 FullResPixelOffset = FullResPixelPos - BaseFullResPixelPos; // Should only be 0 or 1. Taking this offset into account for tracing gives better results.
	// Could do tighter packing of the offset here, but we need at least 16 bits for the normal for good results, so we need to go with RGBA8 anyways.
	OutColor = float4(WorldNormalOct * 0.5f + 0.5f, FullResPixelOffset.x / 255.0f, FullResPixelOffset.y / 255.0f);
	OutDepth = DownsampledDepth;
}

#endif // FirstPersonSelfShadowDownsamplePS


#ifdef FirstPersonSelfShadowTracePS

#include "DynamicLightingCommon.ush"
#include "DeferredShadingCommon.ush"
#include "LightDataUniforms.ush"
#define HZB_TRACE_INCLUDE_FULL_RES_DEPTH 1
#define HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH 1
#include "HZBTracing.ush"
#include "HZB.ush"

#ifndef RAW_FULL_RESOLUTION_FP_SELF_SHADOWS
#define RAW_FULL_RESOLUTION_FP_SELF_SHADOWS 0
#endif

// Remap light shape permutation into light type
#if LIGHT_SOURCE_SHAPE == 0
#define CURRENT_LIGHT_TYPE LIGHT_TYPE_DIRECTIONAL
#elif LIGHT_SOURCE_SHAPE == 2
#define CURRENT_LIGHT_TYPE LIGHT_TYPE_RECT
#else // LIGHT_SOURCE_SHAPE == 1
#define CURRENT_LIGHT_TYPE (IsDeferredSpotlight() ? LIGHT_TYPE_SPOT : LIGHT_TYPE_POINT)
#endif

#if !RAW_FULL_RESOLUTION_FP_SELF_SHADOWS
Texture2D<float4> DownsampledInputsTexture;
Texture2D<float> DownsampledDepthTexture;
#endif

float HZBMaxTraceDistance;
float HZBMaxIterations;
float HZBRelativeDepthThickness;
uint HZBMinimumTracingThreadOccupancy;

struct FFirstPersonScreenTracingInputs
{
	uint2 FullResPixelPos;
	float2 FullResSVPos;
	float DeviceZ;
	float SceneDepth;
	float3 WorldNormal;
};

FFirstPersonScreenTracingInputs GetFirstPersonScreenTracingInputs(uint2 PixelPos)
{
	FFirstPersonScreenTracingInputs Result;

#if RAW_FULL_RESOLUTION_FP_SELF_SHADOWS

	Result.FullResPixelPos = PixelPos;
	Result.FullResSVPos = PixelPos + 0.5f;

	// When running in full resolution, we don't have a stencil bit to let the hardware early-out on non FP pixels, so we do it manually here.
	const bool bIsFirstPerson = IsFirstPersonPixel(Result.FullResPixelPos);
	if (!bIsFirstPerson)
	{
		discard;
	}

	Result.DeviceZ = LookupDeviceZ(Result.FullResPixelPos);
	Result.SceneDepth = ConvertFromDeviceZ(Result.DeviceZ);
	Result.WorldNormal = GetWorldNormal(Result.FullResPixelPos);

#else // RAW_FULL_RESOLUTION_FP_SELF_SHADOWS

	const float4 DownsampledInputs = DownsampledInputsTexture.Load(int3(PixelPos, 0));
	const uint2 PixelOffset = uint2(DownsampledInputs.zw * 255.0f);
	
	Result.FullResPixelPos = min(PixelPos * 2u + PixelOffset, View.ViewRectMinAndSize.zw) + View.ViewRectMinAndSize.xy;
	Result.FullResSVPos = Result.FullResPixelPos + 0.5f;
	Result.DeviceZ = DownsampledDepthTexture.Load(int3(PixelPos, 0)).x;
	Result.SceneDepth = ConvertFromDeviceZ(Result.DeviceZ);
	Result.WorldNormal = OctahedronToUnitVector(DownsampledInputs.xy * 2.0f - 1.0f);
	
#endif // RAW_FULL_RESOLUTION_FP_SELF_SHADOWS
	
	return Result;
}

float4 EncodeOutput(float Shadow, float HitDistance)
{
#if RAW_FULL_RESOLUTION_FP_SELF_SHADOWS
	return EncodeLightAttenuation(Shadow);
#else
	// Store a normalized hit distance (in pixels). 0 means immediate intersection, 1 means miss and 254/255 is the longest representable hit distance.
	const float MaxPixelDistance = 64.0f; // Seems to give decent results with shadows contact hardening up to 64 pixels from the shadow caster.
	HitDistance = (HitDistance >= 0.0f) ? (saturate(HitDistance / MaxPixelDistance) * FP_MAX_VALID_HIT_DISTANCE) : 1.0f;

	return HitDistance;
#endif
}

bool IsLightVisible(FDeferredLightData Light, float3 TranslatedWorldPosition, float3 Normal)
{
	if (!Light.bRadialLight)
	{
		return true;
	}

	const float3 ToLight = Light.TranslatedWorldPosition - TranslatedWorldPosition;
	const float InvDist = rsqrt(dot(ToLight, ToLight));
	const float3 L = ToLight * InvDist;

	const bool bInLightRadius = InvDist >= Light.InvRadius;
	const bool bInLightCone = SpotAttenuationMask(L, -Light.Direction, Light.SpotAngles) > 0.0f;

	bool bInLightRegion = bInLightRadius && bInLightCone;

	if (Light.bRectLight)
	{
		const bool bIsBehindRectLight = dot(ToLight, Light.Direction) < 0.0f;
		if (bIsBehindRectLight)
		{
			bInLightRegion = false;
		}
	}

	return bInLightRegion;
}

float TraceFirstPersonScreenSpaceShadows(FFirstPersonScreenTracingInputs Inputs, FDeferredLightData LightData, float3 TranslatedWorldPosition, out float3 OutHitUVz)
{
	// Bias along the normal to avoid self-intersecting camera facing scene depth texels
	// HZB traversal uses point filtering, so scene depth texels stick out from the plane they are representing
	float NormalBias = 0.0f;
	{
		const float3 CornerTranslatedWorldPosition = SvPositionToTranslatedWorld(float4(Inputs.FullResSVPos + 0.5f, Inputs.DeviceZ, 1.0f));
		// Project texel corner onto normal
		NormalBias = abs(dot(CornerTranslatedWorldPosition - TranslatedWorldPosition, Inputs.WorldNormal)) * 2.0f;
	}

	const float3 TranslatedRayOrigin = TranslatedWorldPosition + NormalBias * Inputs.WorldNormal;
	const float4 RayStartClip = mul(float4(TranslatedRayOrigin, 1.0f), View.TranslatedWorldToClip);

	// Skip screen traces if the normal bias pushed our starting point off-screen, as TraceHZB() doesn't handle this
	if (any(RayStartClip.xy < -RayStartClip.w) || any(RayStartClip.xy > RayStartClip.w))
	{
		return 1.0f;
	}

	float3 RayWorldDirection;
	float MaxWorldTraceDistance;
	if (LightData.bRadialLight)
	{
		const float3 ToLight = LightData.TranslatedWorldPosition - TranslatedRayOrigin;
		const float LightDistance = length(ToLight);
		RayWorldDirection = ToLight / LightDistance;
		MaxWorldTraceDistance = min(LightDistance, HZBMaxTraceDistance);
	}
	else
	{
		RayWorldDirection = LightData.Direction;
		MaxWorldTraceDistance = min(1.0e27f, HZBMaxTraceDistance);
	}

	bool bHit = false;
	bool bUncertain = false;
	float3 OutLastVisibleScreenUV = 0.0f;
	float OutHitTileZ = 0.0f;

	TraceHZB(
		SceneTexturesStruct.SceneDepthTexture,
		ClosestHZBTexture,
		HZBBaseTexelSize,
		HZBUVToScreenUVScaleBias,
		TranslatedRayOrigin,
		RayWorldDirection,
		MaxWorldTraceDistance,
		HZBUvFactorAndInvFactor,
		HZBMaxIterations,
		HZBRelativeDepthThickness,
		0 /*NumThicknessStepsToDetermineCertainty*/,
		HZBMinimumTracingThreadOccupancy,
		bHit,
		bUncertain,
		OutHitUVz,
		OutLastVisibleScreenUV,
		OutHitTileZ);

	return bHit || bUncertain ? 0.0f : 1.0f;
}

void FirstPersonSelfShadowTracePS(
	float4 SVPos : SV_Position,
	out float4 OutColor : SV_Target0)
{
	const FDeferredLightData LightData = InitDeferredLightFromUniforms(CURRENT_LIGHT_TYPE);
	const FFirstPersonScreenTracingInputs Inputs = GetFirstPersonScreenTracingInputs(SVPos.xy);
	const float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(float4(Inputs.FullResSVPos, Inputs.DeviceZ, 1.0f));

	float Shadow = 1.0f;
	float HitDistance = -1.0f;
	if (IsLightVisible(LightData, TranslatedWorldPosition, Inputs.WorldNormal))
	{
		float3 HitUVz;
		Shadow = TraceFirstPersonScreenSpaceShadows(Inputs, LightData, TranslatedWorldPosition, HitUVz);

		if (Shadow < 1.0f)
		{
			HitDistance = distance(HitUVz.xy * View.BufferSizeAndInvSize.xy, Inputs.FullResSVPos);
		}
	}

	OutColor = EncodeOutput(Shadow, HitDistance);
}

#endif // FirstPersonSelfShadowTracePS


#ifdef FirstPersonSelfShadowBlurPS

Texture2D<float4> InputsTexture;
Texture2D<float> DepthTexture;
float InvDepthThreshold;

struct FFirstPersonBlurInputs
{
	float DeviceZ;
	float SceneDepth;
	float HitDistance;
	float ShadowFactor;
};

FFirstPersonBlurInputs LoadInputs(uint2 PixelPos)
{
	FFirstPersonBlurInputs Result = (FFirstPersonBlurInputs)0;
	Result.DeviceZ = DepthTexture.Load(int3(PixelPos, 0)).x;
	if (Result.DeviceZ != FarDepthValue)
	{
		Result.SceneDepth = ConvertFromDeviceZ(Result.DeviceZ);

		const float PackedShadowAndHitDistance = InputsTexture.Load(int3(PixelPos, 0)).x;
		Result.HitDistance = PackedShadowAndHitDistance != 1.0f ? (PackedShadowAndHitDistance / FP_MAX_VALID_HIT_DISTANCE) : -1.0f;
		Result.ShadowFactor = PackedShadowAndHitDistance != 1.0f ? 0.0f : 1.0f;
	}
	return Result;
}

struct FSampleAccumulator
{
	float ShadowSum;
	float HitDistanceSum;
	float ShadowWeightSum;
	float HitDistanceWeightSum;
};

void AccumulateSample(inout FSampleAccumulator Accumulator, FFirstPersonBlurInputs CenterInputs, uint2 SamplePixelPos)
{
	const FFirstPersonBlurInputs SampleInputs = LoadInputs(SamplePixelPos);
	if (SampleInputs.DeviceZ != FarDepthValue)
	{
		const float Scale = InvDepthThreshold;
		const float DepthWeight = min(1.0f, 1.0f / (Scale * abs(CenterInputs.SceneDepth - SampleInputs.SceneDepth) + 1e-4f));
		const float SampleWeight = DepthWeight;

		Accumulator.ShadowSum += SampleInputs.ShadowFactor * SampleWeight;
		Accumulator.ShadowWeightSum += SampleWeight;
		if (SampleInputs.HitDistance > 0.0f)
		{
			Accumulator.HitDistanceSum += SampleInputs.HitDistance * SampleWeight;
			Accumulator.HitDistanceWeightSum += SampleWeight;
		}
	}
}

void FirstPersonSelfShadowBlurPS(
	float4 SVPos : SV_Position,
	out float4 OutColor : SV_Target0)
{
	const uint2 PixelPos = uint2(SVPos.xy);

	const FFirstPersonBlurInputs CenterInputs = LoadInputs(PixelPos);

	// Initialize accumulator with center input
	FSampleAccumulator Accumulator = (FSampleAccumulator)0;
	Accumulator.ShadowSum = CenterInputs.ShadowFactor;
	Accumulator.ShadowWeightSum = 1.0f;
	Accumulator.HitDistanceSum = CenterInputs.HitDistance > 0.0f ? CenterInputs.HitDistance : 0.0f;
	Accumulator.HitDistanceWeightSum = CenterInputs.HitDistance > 0.0f ? 1.0f : 0.0f;
	

	// Accumulate samples in a 3x3 area
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2(-1, -1));
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2( 0, -1));
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2( 1, -1));

	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2(-1,  0));
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2( 1,  0));

	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2(-1,  1));
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2( 0,  1));
	AccumulateSample(Accumulator, CenterInputs, PixelPos + int2( 1,  1));

	const float InvShadowWeightSum = 1.0f / Accumulator.ShadowWeightSum; // ShadowWeightSum should be at least 1.0f due to the center pixel
	const float InvHitDistanceWeightSum = Accumulator.HitDistanceWeightSum > 1e-5f ? (1.0f / Accumulator.HitDistanceWeightSum) : 0.0f; // HitDistanceWeightSum can actually be zero

	const float BlurredShadow = Accumulator.ShadowSum * InvShadowWeightSum;
	const float AverageHitDistance = Accumulator.HitDistanceSum * InvHitDistanceWeightSum;
	const float FilterStrength = saturate(AverageHitDistance);

	// Make shadow contact hardening by lerping between the filtered and unfiltered result. Simple, but works well enough for 3x3.
	const float FinalShadow = lerp(CenterInputs.ShadowFactor, BlurredShadow, FilterStrength);

	OutColor = EncodeLightAttenuation(FinalShadow);
}

#endif // FirstPersonSelfShadowBlurPS


#ifdef FirstPersonSelfShadowUpsamplePS

#include "DeferredShadingCommon.ush"

Texture2D<float> DownsampledDepthTexture;
Texture2D<float4> ShadowFactorsTexture;
float2 DownsampledInvBufferSize;
float InvDepthThreshold;

float4 GatherDepths(float2 UV)
{
	return DownsampledDepthTexture.GatherRed(GlobalBilinearClampedSampler, UV);
}

float4 GatherShadowFactors(float2 UV)
{
	return ShadowFactorsTexture.GatherRed(GlobalBilinearClampedSampler, UV);
}

float4 GetBilinearWeights(float2 DownsampledSVPos)
{
	const float2 DownsampledCoordFrac = frac(DownsampledSVPos - 0.5f);
	const float4 BilinearWeights = float4(
		(1.0f - DownsampledCoordFrac.x) * DownsampledCoordFrac.y,
		DownsampledCoordFrac.x * DownsampledCoordFrac.y,
		DownsampledCoordFrac.x * (1.0f - DownsampledCoordFrac.y),
		(1.0f - DownsampledCoordFrac.x) * (1.0f - DownsampledCoordFrac.y)
	);

	return BilinearWeights;
}

float4 GetDepthWeights(float FullResSceneDepth, float4 DownsampledDeviceZ)
{
	const float4 DownsampledSceneDepths = float4(
		ConvertFromDeviceZ(DownsampledDeviceZ.x),
		ConvertFromDeviceZ(DownsampledDeviceZ.y),
		ConvertFromDeviceZ(DownsampledDeviceZ.z),
		ConvertFromDeviceZ(DownsampledDeviceZ.w));

	const float4 DepthWeights = min(1.0f, 1.0f / (InvDepthThreshold * abs(FullResSceneDepth - DownsampledSceneDepths) + 1e-4f));

	// Downsampled depth has been forced to FarDepthValue for non-FP pixels and we want to ensure they're excluded from the weighting.
	const float4 FirstPersonDepthWeights = select(DownsampledDeviceZ != FarDepthValue, 1.0f, 0.0f);

	return DepthWeights * FirstPersonDepthWeights;
}

void FirstPersonSelfShadowUpsamplePS(
	float4 SVPos : SV_Position,
	out float4 OutColor : SV_Target0)
{
	const bool bIsFirstPerson = IsFirstPersonPixel(uint2(SVPos.xy));
	if (!bIsFirstPerson)
	{
		discard;
	}

	const float2 DownsampledSVPos = (SVPos.xy - View.ViewRectMinAndSize.xy) * 0.5f;
	const float2 DownsampledUV = DownsampledSVPos * DownsampledInvBufferSize;
	const float4 DownsampledDeviceZ = GatherDepths(DownsampledUV);
	
	// Calculate weights
	const float4 DepthWeights = GetDepthWeights(CalcSceneDepth(uint2(SVPos.xy)), DownsampledDeviceZ);
	const float4 BilinearWeights = GetBilinearWeights(DownsampledSVPos);
	
	// Normalize weights or fall back to bilinear
	float4 TotalWeights = DepthWeights * BilinearWeights;
	const float TotalWeightSum = dot(TotalWeights, 1.0f);
	if (TotalWeightSum > 1e-5f)
	{
		TotalWeights /= TotalWeightSum;
	}
	else
	{
		TotalWeights = BilinearWeights;
	}

	// Load low res shadow factors and apply weights
	const float4 LowResShadowFactors = DecodeLightAttenuation(GatherShadowFactors(DownsampledUV));
	const float UpsampledShadowFactor = dot(LowResShadowFactors, TotalWeights);

	OutColor = EncodeLightAttenuation(UpsampledShadowFactor);
}

#endif // FirstPersonSelfShadowUpsamplePS
