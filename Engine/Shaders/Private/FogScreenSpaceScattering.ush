// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FogScreenSpaceScattering.usf: function used by FSSS to approximate multiple scattering.
=============================================================================*/

#include "Common.ush"


// Parameteres representing the fog for the FSSS technique.
struct FFogSSSParameters
{
	float Coverage;
	float SceneDepthMeter;

	float Albedo;
	float Absorption;
	float Scattering;
	float Extinction;
};

FFogSSSParameters EvaluateFSSSParameter(float FogTransmittanceToScene, const float SceneDepthCentimeters)
{
	FFogSSSParameters FSSSParams;

	FSSSParams.SceneDepthMeter = SceneDepthCentimeters * CENTIMETER_TO_METER;

	FSSSParams.Albedo = saturate(1.0f); // Should be computed from fog but it is not always available, e.g. exp heigh fog.

	FSSSParams.Extinction = -log(max(0.0000001f, FogTransmittanceToScene)) / max(FSSSParams.SceneDepthMeter, 0.001f);
	FSSSParams.Scattering = FSSSParams.Extinction * FSSSParams.Albedo;
	FSSSParams.Absorption = FSSSParams.Extinction - FSSSParams.Scattering;

	FSSSParams.Coverage = saturate(1.0f - FogTransmittanceToScene);

	return FSSSParams;
}


// UE version of the estimation of PSD from Premoze et al. [2004] with S = travelled distance.
// Also see "Real-Time Single Scattering in Large-Scale Environments" from Elek et al. [2013].
float FSSS_W(float Absorption, float Scattering, float s)
{
	// Assuming a uniform phase function.
	const float g = 0.0f;
	const float A = 2.0f * Absorption / (3.0f * s);
	const float B = 4.0f / (3.0f * s * s * s * Scattering * (1.0 - g));

	if ((Absorption == 0 && Scattering == 0))
	{
		return 0.0;				// No medium
	}
	else if ((A + B) == 0)
	{
		return MaxHalfFloat;	// Infinite medium
	}

	const float Ws = sqrt(0.5 / (A + B));
	return Ws;
}
