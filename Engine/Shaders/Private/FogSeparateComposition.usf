// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FogSeparateComposition.usf: code related to rendering of fog in separate textures.
	The composition can happen at full or half resolution.
	It also contains code for FSSS (Fog Screen Space Scattering) which is a technique
	used to blur the fog in screen space to approximate multiple scattering.
=============================================================================*/

#include "Common.ush"

#include "FogScreenSpaceScattering.ush"





#ifdef DOWNSAMPLECOLORCS

int SrcMipLevel;
int2 SrcMipResolution;
int2 DstMipResolution;
float BlurMaxExposedLuminance;
float GaussianBlurScale;
SamplerState SourceSampler;
Texture2D<float4> SourceTexture;
RWTexture2D<float4> OutTextureMipColor;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void DownsampleColorCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy >= uint2(DstMipResolution)))
	{
		return;
	}

	float4 Accumulator = 0.0f;
	float2 UVScale = 1.0f / (DstMipResolution * 2);
	float2 SourceCoord = (float2(ThreadId.xy) + 0.5f) * 2.0f;

#if 0
	// Debug 2x2 downsample
	Accumulator += SourceTexture.SampleLevel(SourceSampler, SourceCoord * UVScale, 0); 
#else
	// 4x4 box filter with destination texel at the center with only 4 filtered taps.
	Accumulator += SourceTexture.SampleLevel(SourceSampler, (SourceCoord + float2(-1.0f,-1.0f) * GaussianBlurScale) * UVScale, 0);
	Accumulator += SourceTexture.SampleLevel(SourceSampler, (SourceCoord + float2(-1.0f, 1.0f) * GaussianBlurScale) * UVScale, 0);
	Accumulator += SourceTexture.SampleLevel(SourceSampler, (SourceCoord + float2( 1.0f, 1.0f) * GaussianBlurScale) * UVScale, 0);
	Accumulator += SourceTexture.SampleLevel(SourceSampler, (SourceCoord + float2( 1.0f,-1.0f) * GaussianBlurScale) * UVScale, 0);
	Accumulator  = Accumulator * 0.25f;
#endif

	OutTextureMipColor[uint2(ThreadId.xy)] = Accumulator;
}

#endif // DOWNSAMPLECOLORCS





#ifdef UPSAMPLECOLORCS

// Must match FogSeparateComposition.ush.
// Default: filter source + blend with previous mip
#define MODE_FILTER_AND_BLEND	0
// Copy source to destination without filtering.
#define MODE_COPY_ONLY			1
// Filter one texture into another.
#define MODE_FILTER_ONLY		2
// Blend with another texture.
#define MODE_BLEND_ONLY		3

float UpsampleBlurFactor;
float GaussianBlurScale;
int2 SourceTextureToFilterMipResolution;
float2 SourceTextureToFilterMipResolutionInv;
float BlurMaxExposedLuminance;

SamplerState SourceSampler;

#if PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_AND_BLEND || PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_ONLY || PERMUTATION_UPSAMPLE_MODE == MODE_COPY_ONLY
Texture2D<float4> SourceTextureToFilter;
#endif

#if PERMUTATION_UPSAMPLE_MODE == MODE_BLEND_ONLY
Texture2D<float4> SourceTextureToBlend;
int MipLeveltoBlend0;
int MipLeveltoBlend1;
#endif

#if PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_AND_BLEND
Texture2D<float4> SourceTextureToBlendSRV;
#endif

RWTexture2D<float4> OutTextureMipColor;


#define FSSS_FILTER_1x1		0
#define FSSS_FILTER_3x3		1
#define FSSS_FILTER_5x5		2
#define FSSS_FILTER_5x5_REF	3

#define FSSS_FILTER FSSS_FILTER_5x5



#if FSSS_FILTER == FSSS_FILTER_5x5 || FSSS_FILTER == FSSS_FILTER_5x5_REF

#define FSSS_FILTER_REF_SAMPLE_COUNT 25
#define FSSS_FILTER_SAMPLE_COUNT 9

// 5x5 Gaussian filter with cross weight bilateral samples
static const float2 GaussianOffsets[FSSS_FILTER_REF_SAMPLE_COUNT] = {
	float2(-2.0f, -2.0f), float2(-1.0f, -2.0f), float2(0.0f, -2.0f), float2(1.0f, -2.0f), float2(2.0f, -2.0f),
	float2(-2.0f, -1.0f), float2(-1.0f, -1.0f), float2(0.0f, -1.0f), float2(1.0f, -1.0f), float2(2.0f, -1.0f),
	float2(-2.0f,  0.0f), float2(-1.0f,  0.0f), float2(0.0f,  0.0f), float2(1.0f,  0.0f), float2(2.0f,  0.0f),
	float2(-2.0f,  1.0f), float2(-1.0f,  1.0f), float2(0.0f,  1.0f), float2(1.0f,  1.0f), float2(2.0f,  1.0f),
	float2(-2.0f,  2.0f), float2(-1.0f,  2.0f), float2(0.0f,  2.0f), float2(1.0f,  2.0f), float2(2.0f,  2.0f)
};

static const float GaussianWeights[FSSS_FILTER_REF_SAMPLE_COUNT] = {
	0.00366f, 0.01465f, 0.02564f, 0.01465f, 0.00366f,
	0.01465f, 0.05861f, 0.09523f, 0.05861f, 0.01465f,
	0.02564f, 0.09523f, 0.15018f, 0.09523f, 0.02564f,
	0.01465f, 0.05861f, 0.09523f, 0.05861f, 0.01465f,
	0.00366f, 0.01465f, 0.02564f, 0.01465f, 0.00366f
};

#endif


#if FSSS_FILTER == FSSS_FILTER_5x5
// This fonction should be optimised out to a single float3 when called with static inputs.
// It returns the sample position and weights for NxN samples characterised by O and S. Where S<=2 and O+S<=5 for the 5x5 kernel above.
float3 GetSampleCoordAndWeight(int2 O, int2 S)
{
	float WeightSum = 0.0f;
	float2 Coord = 0.0f;
	UNROLL
	for(int Y = 0; Y < S.y; Y++)
	{
		UNROLL
		for(int X = 0; X < S.x; X++)
		{
			uint Index = (O.y + Y) * 5 + (O.x + X);
			Coord += GaussianOffsets[Index] * GaussianWeights[Index];
			WeightSum += GaussianWeights[Index];
		}
	}
	return float3(Coord / WeightSum, WeightSum);
}
#endif

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void UpsampleColorCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy >= uint2(SourceTextureToFilterMipResolution)))
	{
		return;
	}


#if PERMUTATION_UPSAMPLE_MODE == MODE_COPY_ONLY

	//An optimisation to avoid heavier filtering to full resolution. Only for mip0.
	OutTextureMipColor[uint2(ThreadId.xy)] = SourceTextureToFilter[uint2(ThreadId.xy)];

#elif PERMUTATION_UPSAMPLE_MODE == MODE_BLEND_ONLY

	// Sample previous mip level to bloom blur.
	const float4 FilteredTextureCurrentMipLevel = SourceTextureToBlend.Load(int3(ThreadId.xy, MipLeveltoBlend0));
	const float4 FilteredTexturePreviousMiplevel = SourceTextureToBlend.SampleLevel(SourceSampler, float2(ThreadId.xy + 0.5f) * SourceTextureToFilterMipResolutionInv, MipLeveltoBlend1);

	OutTextureMipColor[uint2(ThreadId.xy)] = lerp(FilteredTextureCurrentMipLevel, FilteredTexturePreviousMiplevel, UpsampleBlurFactor.xxxx);

#elif PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_AND_BLEND || PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_ONLY

	float4 Accumulator = 0.0f;

#if FSSS_FILTER == FSSS_FILTER_1x1

	// 1 sample copy
	Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter[int2(ThreadId.xy)]);

#elif FSSS_FILTER == FSSS_FILTER_3x3

	// 3x3 filter as 4 samples
	const float UpsampleScale = 0.5f * GaussianBlurScale;
	float2 SourceCoord = (float2(ThreadId.xy) + 0.5f) * 0.5f;
	float2 UVScale = 1.0f / (SourceTextureToFilterMipResolution * 0.5f);
	Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter.SampleLevel(SourceSampler, (SourceCoord + (float2(-1.0f,-1.0f) * UpsampleScale)) * UVScale, 0));
	Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter.SampleLevel(SourceSampler, (SourceCoord + (float2(-1.0f, 1.0f) * UpsampleScale)) * UVScale, 0));
	Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter.SampleLevel(SourceSampler, (SourceCoord + (float2( 1.0f, 1.0f) * UpsampleScale)) * UVScale, 0));
	Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter.SampleLevel(SourceSampler, (SourceCoord + (float2( 1.0f,-1.0f) * UpsampleScale)) * UVScale, 0));
	Accumulator *= 0.25;

#elif FSSS_FILTER == FSSS_FILTER_5x5

	// 5x5 filter as 9 samples. 25 texels gathered faster by leveraging bilinear filtering.
	//
	// The samples gathered by each samples for thje 5x5 gaussian kernel above are:
	//  2x2 2x2 1x2
	//  2x2 2x2 1x2
	//  2x1 2x1 1x1
	// Samples represents the final XY offset, while Z represent the gathered Gaussian weight.
	float3 Samples[FSSS_FILTER_SAMPLE_COUNT];

	Samples[0] = GetSampleCoordAndWeight(int2(0, 0), int2(2, 2));
	Samples[1] = GetSampleCoordAndWeight(int2(2, 0), int2(2, 2));
	Samples[2] = GetSampleCoordAndWeight(int2(4, 0), int2(1, 2));

	Samples[3] = GetSampleCoordAndWeight(int2(0, 2), int2(2, 2));
	Samples[4] = GetSampleCoordAndWeight(int2(2, 2), int2(2, 2));
	Samples[5] = GetSampleCoordAndWeight(int2(4, 2), int2(1, 2));

	Samples[6] = GetSampleCoordAndWeight(int2(0, 4), int2(2, 1));
	Samples[7] = GetSampleCoordAndWeight(int2(2, 4), int2(2, 1));
	Samples[8] = GetSampleCoordAndWeight(int2(4, 4), int2(1, 1));

	UNROLL
	for(int i = 0; i < FSSS_FILTER_SAMPLE_COUNT; i++)
	{
		float2 SampleUV = (float2(ThreadId.xy + 0.5f) + Samples[i].xy * GaussianBlurScale) * SourceTextureToFilterMipResolutionInv;
		Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter.SampleLevel(SourceSampler, SampleUV, 0)) * Samples[i].z;
	}

#elif FSSS_FILTER == FSSS_FILTER_5x5_REF

	const int2 ResolutionUpperbound = SourceTextureToFilterMipResolution - 1;
	UNROLL
	for (int i = 0; i < FSSS_FILTER_REF_SAMPLE_COUNT; i++)
	{
		int2 SourceCoordOffset = int2(ThreadId.xy) + int2(GaussianOffsets[i].xy);
		SourceCoordOffset = clamp(0, ResolutionUpperbound, SourceCoordOffset);
		Accumulator += min(BlurMaxExposedLuminance, SourceTextureToFilter[uint2(SourceCoordOffset)]) * GaussianWeights[i];
	}

#endif

#if PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_AND_BLEND

	// Sample previous mip level to bloom blur.
	const float4 PreviousMiplevel = SourceTextureToBlendSRV.SampleLevel(SourceSampler, float2(ThreadId.xy + 0.5f) * SourceTextureToFilterMipResolutionInv, 0);

	// This is a UAV typed load on OutTextureMipColor. See DoesPlatformSupportFSSS for details.
	// Do not pollute alpha as it is specific FSSS that should not be blurred.
	OutTextureMipColor[uint2(ThreadId.xy)] = float4(lerp(Accumulator.rgb, PreviousMiplevel.rgb, UpsampleBlurFactor.xxx).rgb, SourceTextureToFilter[uint2(ThreadId.xy)].a);

#elif PERMUTATION_UPSAMPLE_MODE == MODE_FILTER_ONLY

	// Filter only mode from one texture into another texture of same resolution.
	OutTextureMipColor[uint2(ThreadId.xy)] = Accumulator;

#else
#error unkown mode.
#endif

#endif
}

#endif // UPSAMPLECOLORCS





#ifdef HALFRESOLUTIONUPSAMPLERPS

#include "FogSeparateComposition.ush"

#if !PERMUTATION_UPSAMPLE_FULL_RESOLUTION
Texture2D<float> HalfResDepthTexture;
SamplerState HalfResDepthSampler;
Texture2D<float> FullResDepthTexture;
SamplerState FullResDepthSampler;
#endif

void HalfResolutionUpsamplerPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
#if PERMUTATION_UPSAMPLE_FULL_RESOLUTION

	float2 FullResLocalCoord = SvPosition.xy - View.ViewRectMinAndSize.xy;

	OutColor = FSSSFullResolutionSample(FullResLocalCoord);

#else // PERMUTATION_UPSAMPLE_FULL_RESOLUTION

	const float FullResDepth = ConvertFromDeviceZ(FullResDepthTexture[SvPosition.xy]);

	float2 FullResLocalCoord = SvPosition.xy - View.ViewRectMinAndSize.xy;

	float2 HalfResLocalCoord = FullResLocalCoord * FSSSGlobals.FullResToFogResScale;

	// Debug raw buffer
#if 0
	SourceTextureColor = FSSSGlobals.FogSeparateCompositionTexture0[HalfResLocalCoord].rgba;
	SourceTextureColor2 = FSSSGlobals.FogSeparateCompositionTexture1[HalfResLocalCoord].r;
	OutColor = FSSSCombineSourceTextures(SourceTextureColor, SourceTextureColor2);
	return;
#endif

	float4 SourceTextureColor = 0;
	float SourceTextureColor2 = 0;


	int2 HalfResLocalCoordI = int2(HalfResLocalCoord);
	const int OffsetX = (frac(HalfResLocalCoord.x) > 0.5f ? 1 : -1);
	const int OffsetY = (frac(HalfResLocalCoord.y) > 0.5f ? 1 : -1);





	// TODO try GatherRed, but need to make sure UV fetch the 4 valid texel
	float4 HalfResDepths;
	HalfResDepths.x = ConvertFromDeviceZ(HalfResDepthTexture[clamp(FSSSGlobals.HalfResDepthMinMaxCoord.xy + HalfResLocalCoordI							, FSSSGlobals.HalfResDepthMinMaxCoord.xy, FSSSGlobals.HalfResDepthMinMaxCoord.zw)]);
	HalfResDepths.y = ConvertFromDeviceZ(HalfResDepthTexture[clamp(FSSSGlobals.HalfResDepthMinMaxCoord.xy + HalfResLocalCoordI + int2(OffsetX,	0)		, FSSSGlobals.HalfResDepthMinMaxCoord.xy, FSSSGlobals.HalfResDepthMinMaxCoord.zw)]);
	HalfResDepths.z = ConvertFromDeviceZ(HalfResDepthTexture[clamp(FSSSGlobals.HalfResDepthMinMaxCoord.xy + HalfResLocalCoordI + int2(OffsetX,	OffsetY), FSSSGlobals.HalfResDepthMinMaxCoord.xy, FSSSGlobals.HalfResDepthMinMaxCoord.zw)]);
	HalfResDepths.w = ConvertFromDeviceZ(HalfResDepthTexture[clamp(FSSSGlobals.HalfResDepthMinMaxCoord.xy + HalfResLocalCoordI + int2(0,		OffsetY), FSSSGlobals.HalfResDepthMinMaxCoord.xy, FSSSGlobals.HalfResDepthMinMaxCoord.zw)]);

	float4 DeltaDepths = abs(HalfResDepths - FullResDepth.xxxx);

	float LargestDepthDiff = max(max(DeltaDepths.x, DeltaDepths.y), max(DeltaDepths.z, DeltaDepths.w)) - min(min(DeltaDepths.x, DeltaDepths.y), min(DeltaDepths.z, DeltaDepths.w));
	const float DeltaToUpsamplingAtDepth1 = 0.2;
//	if (LargestDepthDiff > DeltaToUpsamplingAtDepth1 * FullResDepth)
	if (LargestDepthDiff > 100.0f)
	{
		// Nearest upsampling

		float MinDepthDelta = 0;
		int2 UpsamplingCoord = 0;

		//if (DeltaDepths.x < MinDepthDelta) // always initialise with first input
		{
			MinDepthDelta = DeltaDepths.x;
			UpsamplingCoord = HalfResLocalCoordI + int2(0,			0);
		}

		if (DeltaDepths.y < MinDepthDelta)
		{
			MinDepthDelta = DeltaDepths.y;
			UpsamplingCoord = HalfResLocalCoordI + int2(OffsetX,	0);
		}

		if (DeltaDepths.z < MinDepthDelta)
		{
			MinDepthDelta = DeltaDepths.z;
			UpsamplingCoord = HalfResLocalCoordI + int2(OffsetX,	OffsetY);
		}

		if (DeltaDepths.w < MinDepthDelta)
		{
			MinDepthDelta = DeltaDepths.w;
			UpsamplingCoord = HalfResLocalCoordI + int2(0,			OffsetY);
		}

		SourceTextureColor = FSSSGlobals.FogSeparateCompositionTexture0[UpsamplingCoord].rgba;
		SourceTextureColor2 = FSSSGlobals.FogSeparateCompositionTexture1[UpsamplingCoord].r;
		OutColor = FSSSCombineSourceTextures(SourceTextureColor, SourceTextureColor2);
		return;
		//HalfResLocalCoord = UpsamplingCoord;
	}



	SourceTextureColor = FSSSGlobals.FogSeparateCompositionTexture0.SampleLevel(FSSSGlobals.FogSeparateCompositionTexture0Sampler, HalfResLocalCoord * FSSSGlobals.TexelCoordToUVs, 0).rgba;
	SourceTextureColor2 = FSSSGlobals.FogSeparateCompositionTexture1.SampleLevel(FSSSGlobals.FogSeparateCompositionTexture1Sampler, HalfResLocalCoord * FSSSGlobals.TexelCoordToUVs, 0).r;
	OutColor = FSSSCombineSourceTextures(SourceTextureColor, SourceTextureColor2);



	// DEBUG

//	OutColor = frac(View.GameTime)>0.5 ? float4( saturate(FullResDepth.xxx*0.001), 0) : float4(0, 0, 0, 1);

//	float Test = frac(View.GameTime*0.5);
//	OutColor = float4(0, 0, 0, 1);
//	if (Test < 0.5)
//	{
//		Test *= 2;
//		if (Test < 1.0 / 4.0)
//			OutColor = float4(saturate(HalfResDepths.xxx * 0.001), 0);
//		else if (Test < 2.0 / 4.0)
//			OutColor = float4(saturate(HalfResDepths.yyy * 0.001), 0);
//		else if (Test < 3.0 / 4.0)
//			OutColor = float4(saturate(HalfResDepths.zzz * 0.001), 0);
//		else //if (Test * 0.5 < 0.5 * 2.0 / 4.0)
//			OutColor = float4(saturate(HalfResDepths.www * 0.001), 0);
//	}

#endif // PERMUTATION_UPSAMPLE_FULL_RESOLUTION

}


#endif // HALFRESOLUTIONUPSAMPLERPS
