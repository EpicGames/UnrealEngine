// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FogSeparateComposition.ush: Common code for composing fog from separate composition textures
	This code can be used by multiple passes (fog upsampling and cloud composition) to apply
	fog textures onto the scene.
=============================================================================*/

#pragma once



float4 FSSSCombineSourceTextures(float4 SourceTextureColor, float SourceTextureColor2)
{
	return float4(SourceTextureColor.rgb, SourceTextureColor2);
}

float4 FSSSFullResolutionSample(float2 FullResLocalCoord)
{
	const float2 Data1 = FSSSGlobals.FogSeparateCompositionTexture1[FullResLocalCoord].xy;
	const float W = Data1.x;
	if (W <= 0.0f)
	{
		clip(-1);
		// Such value will lead to no difference when upsampling
		return 0.0f;
	}

	// Always applied transmitance unblurred from mip0, because transmittance do not scatter.
	const float FogTransmittanceMip0 = Data1.y;
	const float FogCoverageMip0 = saturate(1.0f - FogTransmittanceMip0);

	// When optimise  mip level 0 blurre away to not pay the cost and also have a sharp fog version always available.
	const float MinBlurMip = 1.0f;

	// We clamp mips to the valid range and we weight by coverage of HeightFog+VFog+LFV only, not cloud.
	// This is to avoid self blurring of volumetric cloud over infinite depth causing W to tend to infinite, even with light heightfog.
	// This also prevent fog to blur/leak to appear over sharp object at in front of it when no fog is present on them. This automatically fades out the scattering.
	const float Mip = FogCoverageMip0 * FSSSGlobals.FSSSSpreadScale * clamp(log2(W / 2), MinBlurMip, FSSSGlobals.FSSSMaxMip);

	// This is a custom fade based on W for the clip above to note cause discontinuity.
	const float FadeBasedOnW = saturate(W);

	// Select the scattered color using the mip level corresponding to the blur. The cloud transmittance must be fetched from Mip0 to not be blurred.
	float  CloudTransmittance		= FSSSGlobals.FogSeparateCompositionTexture0.SampleLevel(FSSSGlobals.FogSeparateCompositionTexture0Sampler, FullResLocalCoord * FSSSGlobals.TexelCoordToUVs, 0).a;
	float3 SourceTextureColorMip	= FSSSGlobals.FogSeparateCompositionTexture0.SampleLevel(FSSSGlobals.FogSeparateCompositionTexture0Sampler, FullResLocalCoord * FSSSGlobals.TexelCoordToUVs, Mip).rgb;

	return float4(
		lerp(0, SourceTextureColorMip, FadeBasedOnW),
		FogTransmittanceMip0 * CloudTransmittance);
}


