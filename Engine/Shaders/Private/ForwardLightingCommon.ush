// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ForwardLightingCommon.ush
=============================================================================*/

#define NON_DIRECTIONAL_DIRECT_LIGHTING (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL)
#define SUPPORT_CONTACT_SHADOWS (MATERIAL_CONTACT_SHADOWS && !FORWARD_SHADING)

#include "DeferredLightingCommon.ush"
#include "LightGridCommon.ush"
#include "LightData.ush"

#define FILTER_DIRECTIONAL_LIGHT_SHADOWING 1
#include "ForwardShadowingCommon.ush"

#ifndef DISABLE_FORWARD_DIRECTIONAL_LIGHT_SHADOW
#define DISABLE_FORWARD_DIRECTIONAL_LIGHT_SHADOW 0
#endif

#if USE_HAIR_COMPLEX_TRANSMITTANCE
#include "HairStrands/HairStrandsCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceDualScattering.ush"
#endif

#ifndef VIRTUAL_SHADOW_MAP
#define VIRTUAL_SHADOW_MAP 0
#endif

#ifndef MATERIAL_ENABLE_TRANSLUCENT_LOCAL_LIGHT_SHADOW
#define MATERIAL_ENABLE_TRANSLUCENT_LOCAL_LIGHT_SHADOW 0
#endif
#ifndef MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_LOCAL_LIGHT_SHADOW
#define MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_LOCAL_LIGHT_SHADOW 0
#endif
#ifndef MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_DIRECTIONAL_LIGHT_SHADOW
#define MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_DIRECTIONAL_LIGHT_SHADOW 0
#endif

#if VIRTUAL_SHADOW_MAP
#include "VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#if SUPPORT_VSM_FOWARD_QUALITY > 0 || MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_LOCAL_LIGHT_SHADOW || MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_DIRECTIONAL_LIGHT_SHADOW
#include "VirtualShadowMaps/VirtualShadowMapProjectionFilter.ush"
#include "VirtualShadowMaps/VirtualShadowMapProjectionDirectional.ush"
#include "VirtualShadowMaps/VirtualShadowMapProjectionSpot.ush"
#endif // SUPPORT_VSM_FOWARD_QUALITY
#endif // VIRTUAL_SHADOW_MAP

#ifndef ENABLE_FORWARD_CLOUD_SHADOW
#define ENABLE_FORWARD_CLOUD_SHADOW 0
#endif

float PrevSceneColorPreExposureInv;

void UpdateNearestSample(float Z, float2 UV, float FullResZ, inout float MinDist, inout float2 NearestUV)
{
    float DepthDelta = abs(Z - FullResZ);

	FLATTEN
    if (DepthDelta < MinDist)
    {
        MinDist = DepthDelta;
        NearestUV = UV;
    }
}

float2 CalculateNearestResolvedDepthScreenUV(float2 ScreenUV, float SceneDepth)
{
	float2 EffectiveScreenUV = ScreenUV;

	if (View.NumSceneColorMSAASamples > 1)
	{
		int2 IntScreenUV = int2(trunc(ScreenUV * View.BufferSizeAndInvSize.xy));

		float DeferredShadowingDepth = ConvertFromDeviceZ(OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(IntScreenUV, 0)).r);
		float RelativeDepthThreshold = .01f;

		// Fragment depth doesn't match what we used for deferred shadowing, search neighbors in cross pattern
		// Even with this nearest depth upsampling there can be edge artifacts from deferred shadowing, 
		// Since depth testing and stencil testing used during shadow projection are done per-sample and we're fetching from the resolved light attenuation
		if (abs(DeferredShadowingDepth - SceneDepth) / SceneDepth > RelativeDepthThreshold)
		{
			float2 TexelSize = View.BufferSizeAndInvSize.zw;
			float MinDist = 1.e8f;

			float2 LeftUV = ScreenUV + float2(-TexelSize.x, 0);
			float LeftDepth = ConvertFromDeviceZ(OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(IntScreenUV.x - 1, IntScreenUV.y, 0)).r);
			UpdateNearestSample(LeftDepth, LeftUV, SceneDepth, MinDist, EffectiveScreenUV);

			float2 UpUV = ScreenUV + float2(0, TexelSize.y);
			float UpDepth = ConvertFromDeviceZ(OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(IntScreenUV.x, IntScreenUV.y + 1, 0)).r);
			UpdateNearestSample(UpDepth, UpUV, SceneDepth, MinDist, EffectiveScreenUV);

			float2 RightUV = ScreenUV + float2(TexelSize.x, 0);
			float RightDepth = ConvertFromDeviceZ(OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(IntScreenUV.x + 1, IntScreenUV.y, 0)).r);
			UpdateNearestSample(RightDepth, RightUV, SceneDepth, MinDist, EffectiveScreenUV);

			float2 BottomUV = ScreenUV + float2(0, -TexelSize.y);
			float BottomDepth = ConvertFromDeviceZ(OpaqueBasePass.ResolvedSceneDepthTexture.Load(int3(IntScreenUV.x, IntScreenUV.y - 1, 0)).r);
			UpdateNearestSample(BottomDepth, BottomUV, SceneDepth, MinDist, EffectiveScreenUV);
		}
	}

	return EffectiveScreenUV;
}

float4 GetForwardDynamicShadowFactors(float2 ScreenUV)
{
	int2 IntScreenUV = int2(trunc(ScreenUV * View.BufferSizeAndInvSize.xy));
	float4 Value = 1.0f;
	// Avoid sampling the white texture fallback because Load returns 0 when out-of-bound.
	BRANCH
	if (OpaqueBasePass.UseForwardScreenSpaceShadowMask)
	{
		Value = OpaqueBasePass.ForwardScreenSpaceShadowMaskTexture.Load(int3(IntScreenUV, 0));
	}
	return DecodeLightAttenuation(Value);
}

float GetIndirectOcclusion(float2 ScreenUV, bool bHasDynamicIndirectShadowCasterRepresentation)
{
	float IndirectOcclusion;

	uint IndirectOcclusionWidth, IndirectOcclusionHeight;
	OpaqueBasePass.IndirectOcclusionTexture.GetDimensions(IndirectOcclusionWidth, IndirectOcclusionHeight);

    int2 IntScreenUV = int2(trunc(ScreenUV * float2(IndirectOcclusionWidth, IndirectOcclusionHeight)));
	IndirectOcclusion = OpaqueBasePass.IndirectOcclusionTexture.Load(int3(IntScreenUV, 0)).x;

	// Reduce self shadowing intensity on characters (reuse distance field bit, really should be HasCapsuleShadowRepresentation)
	IndirectOcclusion = lerp(1, IndirectOcclusion, bHasDynamicIndirectShadowCasterRepresentation ? View.IndirectCapsuleSelfShadowingIntensity : 1);

	return IndirectOcclusion;
}

FDeferredLightingSplit GetForwardDirectLightingSplit(
	uint2 PixelPos,
	uint GridIndex, float3 TranslatedWorldPosition, float3 CameraVector, FGBufferData GBufferData, float2 ScreenUV, uint PrimitiveId, uint EyeIndex, bool bExcludeMegaLights, float Dither, 
	float InDirectionalLightCloudShadow, float3 InDirectionalLightAtmosphereTransmittance, inout float OutDirectionalLightShadow,
	bool bSeparateMainDirLightLuminance, inout float3 SeparatedMainDirLightLuminance, bool bSkipDirLightVirtualShadowMapEvaluation)
{
	float4 DynamicShadowFactors = 1;

	#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
		DynamicShadowFactors = GetForwardDynamicShadowFactors(ScreenUV);
	#endif

	FDeferredLightingSplit DirectLighting;
	DirectLighting.DiffuseLighting = 0;
	DirectLighting.SpecularLighting = 0;
	DirectLighting.LightingLuminance = 0;

	float SpecularScale = 1;
#if TRANSLUCENCY_ANY_VOLUMETRIC
	// No specular on volumetric translucency lighting modes
	SpecularScale = 0;
#endif

	uint LightingChannelMask = GetPrimitive_LightingChannelMask(PrimitiveId);

	const FDirectionalLightData DirectionalLightData = GetDirectionalLightData();

	BRANCH
	if (DirectionalLightData.HasDirectionalLight
		&& (!bExcludeMegaLights || !DirectionalLightData.bHandledByMegaLights)
#if MATERIALBLENDING_ANY_TRANSLUCENT
		&& DirectionalLightData.bAffectsTranslucentLighting
#endif
		)
	{
		half4 PreviewShadowMapChannelMask = 1;
		uint  DirLightingChannelMask = LIGHTING_CHANNEL_MASK;
		FDeferredLightData LightData = ConvertToDeferredLight(DirectionalLightData, SpecularScale, PreviewShadowMapChannelMask, DirLightingChannelMask);
		#if USE_HAIR_COMPLEX_TRANSMITTANCE
		if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
		{
			LightData.HairTransmittance = EvaluateDualScattering(GBufferData, -CameraVector, LightData.Direction);
		}
		#endif

		// We want to force the directional light shadow when using water material to see shadow on the water. This could be an option later.
		#if DISABLE_FORWARD_DIRECTIONAL_LIGHT_SHADOW
			float4 LightAttenuation = float4(1, 1, 1, 1);
		#elif ((MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
			float DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);

			// In the forward shading path we can't separate per-object shadows from CSM, since we only spend one light attenuation channel per light
			// If CSM is enabled (distance fading to precomputed shadowing is active), treat all of our dynamic shadowing as whole scene shadows that will be faded out at the max CSM distance
			// If CSM is not enabled, allow our dynamic shadowing to coexist with precomputed shadowing
			float PerObjectShadowing = LightData.DistanceFadeMAD.y < 0.0f ? 1.0f : DynamicShadowing;
			float WholeSceneShadowing = LightData.DistanceFadeMAD.y < 0.0f ? DynamicShadowing : 1.0f;
		
			float4 LightAttenuation = float4(WholeSceneShadowing.xx, PerObjectShadowing.xx);
		#else
			LightData.ShadowedBits = 1;
			LightData.ShadowMapChannelMask.x = 1;
			#if TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
				GBufferData.PrecomputedShadowFactors.x = ComputeDirectionalLightStaticShadowing(TranslatedWorldPosition).x;
			#else
				GBufferData.PrecomputedShadowFactors.x = 1;
			#endif
			bool bUnused = false;
			float DynamicShadowFactor = ComputeDirectionalLightDynamicShadowing(TranslatedWorldPosition, GBufferData.Depth, bUnused);
			
			#if VIRTUAL_SHADOW_MAP
			BRANCH
			if ( !bSkipDirLightVirtualShadowMapEvaluation && ForwardLightStruct.DirectionalLightVSM != INDEX_NONE )
			{
				#if SUPPORT_VSM_FOWARD_QUALITY == 1 || MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_DIRECTIONAL_LIGHT_SHADOW
				{
					const FLightShaderParameters DirectionalLight = ConvertToLightShaderParameters(LightData);

					const float ScreenRayLengthWorld = VirtualShadowMap.ScreenRayLength * View.ClipToView[1][1] * GBufferData.Depth;
					float SMRTRayOffset = ScreenRayLengthWorld;

					FVirtualShadowMapSampleResult VirtualShadowMapSample = TraceDirectional(
						ForwardLightStruct.DirectionalLightVSM,
						DirectionalLight,
						PixelPos,
						GBufferData.Depth,
						TranslatedWorldPosition,
						SMRTRayOffset,
						Dither,
						GBufferData.WorldNormal);

					FilterVirtualShadowMapSampleResult(PixelPos, VirtualShadowMapSample);

					DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
				}
				#else
				{
					FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapDirectional( ForwardLightStruct.DirectionalLightVSM, TranslatedWorldPosition );
					DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
				}
				#endif
			}
			#endif

			float4 LightAttenuation = float4(DynamicShadowFactor.x, DynamicShadowFactor.x, 1, 1);
		#endif

		FDeferredLightingSplit NewLighting = GetDynamicLightingSplit(TranslatedWorldPosition, -CameraVector, GBufferData, 1, LightData, LightAttenuation, Dither, uint2(0,0), OutDirectionalLightShadow);

		FLATTEN
		if (DirLightingChannelMask & LightingChannelMask)
		{
		#if ENABLE_FORWARD_CLOUD_SHADOW
			// This cannot go into LightAttenuation due to some distance based attenuation math being applied.
			NewLighting.DiffuseLighting *= InDirectionalLightCloudShadow;
			NewLighting.SpecularLighting *= InDirectionalLightCloudShadow;
			NewLighting.LightingLuminance *= InDirectionalLightCloudShadow;
		#endif
			NewLighting.DiffuseLighting.rgb *= InDirectionalLightAtmosphereTransmittance;
			NewLighting.SpecularLighting.rgb *= InDirectionalLightAtmosphereTransmittance;
			NewLighting.LightingLuminance *= Luminance(InDirectionalLightAtmosphereTransmittance);

			if (bSeparateMainDirLightLuminance)
			{
				SeparatedMainDirLightLuminance += NewLighting.DiffuseLighting.rgb;
				SeparatedMainDirLightLuminance += NewLighting.SpecularLighting.rgb;
			}
			else
			{
				DirectLighting.DiffuseLighting += NewLighting.DiffuseLighting;
				DirectLighting.SpecularLighting += NewLighting.SpecularLighting;
				DirectLighting.LightingLuminance += NewLighting.LightingLuminance;
			}
		}
	}

#if !DISABLE_FORWARD_LOCAL_LIGHTS

	const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

	uint NumLightsInGridCell = CulledLightsGridHeader.NumLights;

	if (bExcludeMegaLights)
	{
		NumLightsInGridCell = CulledLightsGridHeader.NumLights - CulledLightsGridHeader.NumMegaLights;
	}

	// Limit max to ForwardLightStruct.NumLocalLights.
	// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
	NumLightsInGridCell = min(NumLightsInGridCell, GetMaxLightsPerCell());

	LOOP
	for (uint GridLightListIndex = 0; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
	{
		half4 PreviewShadowMapChannelMask = 1;
		uint  LocalLightingChannelMask = LIGHTING_CHANNEL_MASK;
		const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);
		
	#if MATERIALBLENDING_ANY_TRANSLUCENT
		if(UnpackAffectsTranslucentLighting(LocalLight) == 0)
		{
			continue;
		}
	#endif

		FDeferredLightData LightData = ConvertToDeferredLight(LocalLight, SpecularScale, PreviewShadowMapChannelMask, LocalLightingChannelMask);

		// Rect light optionally supported in forward.
		LightData.bRectLight = LightData.bRectLight && SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT;
		#if USE_HAIR_COMPLEX_TRANSMITTANCE
		if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
		{
			LightData.HairTransmittance = EvaluateDualScattering(GBufferData, -CameraVector, LightData.Direction);
		}
		#endif
		#if USE_IES_PROFILE
		LightData.Color *= ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);
		#endif

		float DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);
		float4 LightAttenuation = float4(1, 1, DynamicShadowing.x, DynamicShadowing.x);

		#if VIRTUAL_SHADOW_MAP && (SUPPORT_SHADOWED_LOCAL_LIGHT_ON_FORWARD_LIT_TRANSLUCENT || MATERIAL_ENABLE_TRANSLUCENT_LOCAL_LIGHT_SHADOW)
		BRANCH
		if (LocalLight.Internal.VirtualShadowMapId != INDEX_NONE)
		{
			float DynamicShadowFactor = 1.f;
			LightData.ShadowedBits = 1;
			#if SUPPORT_VSM_FOWARD_QUALITY == 1 || MATERIAL_ENABLE_TRANSLUCENT_HIGH_QUALITY_LOCAL_LIGHT_SHADOW
			{
				const FLightShaderParameters LocalLightShaderParameters = ConvertToLightShaderParameters(LightData);

				const float ScreenRayLengthWorld = VirtualShadowMap.ScreenRayLength * View.ClipToView[1][1] * GBufferData.Depth;
				float SMRTRayOffset = ScreenRayLengthWorld;

				FVirtualShadowMapSampleResult VirtualShadowMapSample = TraceLocalLight(
					LocalLight.Internal.VirtualShadowMapId,
					LocalLightShaderParameters,
					PixelPos,
					GBufferData.Depth,
					TranslatedWorldPosition,
					SMRTRayOffset,
					Dither,
					GBufferData.WorldNormal);

				FilterVirtualShadowMapSampleResult(PixelPos, VirtualShadowMapSample);

				DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
			}
			#else
			{
				FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapLocal(LocalLight.Internal.VirtualShadowMapId, TranslatedWorldPosition);
				DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
			}
			#endif

			LightAttenuation = DynamicShadowFactor;
		}
		#endif

		float SurfaceShadow = 1.0f;
		FDeferredLightingSplit NewLighting = GetDynamicLightingSplit(TranslatedWorldPosition, -CameraVector, GBufferData, 1, LightData, LightAttenuation, Dither, uint2(0,0), SurfaceShadow);

		FLATTEN
		if (LocalLightingChannelMask & LightingChannelMask)
		{
			DirectLighting.DiffuseLighting += NewLighting.DiffuseLighting;
			DirectLighting.SpecularLighting += NewLighting.SpecularLighting;
		}
	}

#endif

	// Zero out direct lighting if show flag is disabled
	if (ForwardLightStruct.DirectLightingShowFlag == 0)
	{
		DirectLighting.DiffuseLighting = 0.0f;
		DirectLighting.SpecularLighting = 0.0f;
	}

	return DirectLighting;
}

float3 GetForwardDirectLightingForVertexLighting(uint GridIndex, float3 TranslatedWorldPosition, float SceneDepth, float3 WorldNormal, uint EyeIndex, float InDirectionalLightCloudShadow)
{
	float3 DirectLighting = 0;
	// Using white for diffuse color, real diffuse color will be incorporated per-pixel
	float3 DiffuseColor = 1.0f;

	const FDirectionalLightData DirectionalLightData = GetDirectionalLightData();

	BRANCH
	if (DirectionalLightData.HasDirectionalLight)
	{
		float3 N = WorldNormal;
		float3 L = DirectionalLightData.DirectionalLightDirection;
		float NoL = saturate(dot(N, L));

		float3 LightColor = DirectionalLightData.DirectionalLightColor;
	
		#if NON_DIRECTIONAL_DIRECT_LIGHTING
			NoL = 1.0f;
		#endif

		float ShadowFactor = ComputeDirectionalLightStaticShadowing(TranslatedWorldPosition);
		bool bUnused = false;
		ShadowFactor *= ComputeDirectionalLightDynamicShadowing(TranslatedWorldPosition, SceneDepth, bUnused);

	#if ENABLE_FORWARD_CLOUD_SHADOW
		ShadowFactor *= InDirectionalLightCloudShadow;
	#endif

		// No specular for vertex lighting
		float3 DiffuseLighting = Diffuse_Lambert(DiffuseColor);
		DirectLighting += LightColor * (NoL * ShadowFactor) * DiffuseLighting;
	}
	
	const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

	// Limit max to ForwardLightStruct.NumLocalLights.
	// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
	const uint NumLightsInGridCell = min(CulledLightsGridHeader.NumLights, GetMaxLightsPerCell());

	LOOP
	for (uint GridLightListIndex = 0; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
	{
		const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);
		const FSimpleDeferredLightData LightData = ConvertToSimpleLight(LocalLight);

		// No specular for vertex lighting
		float3 CameraVector = 0;
		float3 SpecularColor = 0;
		float Roughness = 1.0f;
		DirectLighting += GetSimpleDynamicLighting(TranslatedWorldPosition, CameraVector, WorldNormal, 1, DiffuseColor, SpecularColor, Roughness, LightData);
	}

	// Zero out direct lighting if show flag is disabled
	if (ForwardLightStruct.DirectLightingShowFlag == 0)
	{
		DirectLighting = 0.0f;
	}

	return DirectLighting;
}

uint MortonCode( uint x )
{
	//x = (x ^ (x <<  8)) & 0x00ff00ff;
	//x = (x ^ (x <<  4)) & 0x0f0f0f0f;
	x = (x ^ (x <<  2)) & 0x33333333;
	x = (x ^ (x <<  1)) & 0x55555555;
	return x;
}

// Translucency Surface per-pixel uses blended reflection captures by default in the deferred renderer
// Have to opt-in to blended reflection captures in the forward renderer
#define USE_BLENDED_REFLECTION_CAPTURES_DEFERRED (!FORWARD_SHADING && (TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME))
#define USE_BLENDED_REFLECTION_CAPTURES_FORWARD (FORWARD_SHADING && MATERIAL_HQ_FORWARD_REFLECTION_CAPTURES)
#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES ((USE_BLENDED_REFLECTION_CAPTURES_DEFERRED || USE_BLENDED_REFLECTION_CAPTURES_FORWARD) && FEATURE_LEVEL >= FEATURE_LEVEL_SM5)
// REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND covers forward rendered material for deferred and forward shading pathes.
#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND (MATERIAL_FORWARD_BLENDS_SKYLIGHT_CUBEMAPS || FORWARD_SHADING_FORCES_SKYLIGHT_CUBEMAPS_BLENDING)
#define REFLECTION_COMPOSITE_HAS_BOX_CAPTURES 1
#define REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES 1
#include "ReflectionEnvironmentComposite.ush"

half3 GetImageBasedReflectionSpecular(FMaterialPixelParameters MaterialParameters, float3 RayDirection, half Roughness, half IndirectIrradiance, uint GridIndex, int SingleCaptureIndex)
{
	float3 SpecularIBL;
	bool bUseLumenFrontLayerReflection = false;

#if (TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME) && MATERIALBLENDING_ANY_TRANSLUCENT && PROJECT_SUPPORTS_LUMEN && !FORWARD_SHADING
	
	bUseLumenFrontLayerReflection = UseFrontLayerReflection(MaterialParameters.ViewBufferUV, MaterialParameters.ScreenPosition.w, MATERIAL_TRANSLUCENT_PASS_AFTERMOTIONBLUR);

	FRadianceCacheCoverage LumenRadianceCacheCoverage = InitRadianceCacheCoverage();
	LumenRadianceCacheCoverage.bValid = false;

	// Lumen Radiance Cache for reflections on translucency
	if (FinalProbeResolution > 0 && !bUseLumenFrontLayerReflection)
	{
		float ClipmapDitherRandom = InterleavedGradientNoise(MaterialParameters.SvPosition.xy, View.StateFrameIndexMod8);
		LumenRadianceCacheCoverage = GetRadianceCacheCoverage(GetTranslatedWorldPosition(MaterialParameters), RayDirection, ClipmapDitherRandom);
	}

	if (bUseLumenFrontLayerReflection)
	{
		SpecularIBL = SampleFrontLayerReflection(MaterialParameters.ViewBufferUV);
	}
	else if (LumenRadianceCacheCoverage.bValid)
	{
		float ConeHalfAngle = 0;
		SpecularIBL = SampleRadianceCacheInterpolated(LumenRadianceCacheCoverage, GetTranslatedWorldPosition(MaterialParameters), RayDirection, ConeHalfAngle, /*RandomScalarForStochasticInterpolation*/ 0.5f).Radiance;
	}
	// Fallback to unshadowed skylight if no valid Radiance Cache
	else
#endif
	{
		FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);

		const bool bCompositeSkylight = true;
		SpecularIBL = CompositeReflectionCapturesAndSkylightTWS(
			1.0f, 
			MaterialParameters.WorldPosition_CamRelative, 
			RayDirection, 
			Roughness, 
			IndirectIrradiance, 
			1.0f, 
			0.0f, 
			CulledReflectionCapturesGridHeader.NumReflectionCaptures, 
			CulledReflectionCapturesGridHeader.DataStartIndex, 
			SingleCaptureIndex,
			bCompositeSkylight);
	}

#if MATERIAL_SSR && !FORWARD_SHADING
	if( View.CameraCut == 0 && TranslucentBasePass.SSRQuality > 0 && !bUseLumenFrontLayerReflection)
	{
		float StepOffset = InterleavedGradientNoise( MaterialParameters.SvPosition.xy, View.StateFrameIndexMod8 );
		StepOffset -= 0.5;

		bool bDebugPrint = false;

		float3 HitUVz;
		float Level = 0;

		bool bHit = RayCast(
			TranslucentBasePass.HZBTexture, TranslucentBasePass.HZBSampler,
			MaterialParameters.WorldPosition_CamRelative, RayDirection, Roughness, MaterialParameters.ScreenPosition.w,
			12, StepOffset,
			TranslucentBasePass.HZBUvFactorAndInvFactor,
			bDebugPrint,
			HitUVz,
			Level
		);

		BRANCH if( bHit )
		{
			float2 SampleUV;
			float Vignette;
			ReprojectHit(TranslucentBasePass.PrevScreenPositionScaleBias, HitUVz, SampleUV, Vignette);

			SampleUV = clamp(SampleUV, TranslucentBasePass.PrevSceneColorBilinearUVMin, TranslucentBasePass.PrevSceneColorBilinearUVMax);

			float4 SSR = SampleScreenColor( 
				TranslucentBasePass.PrevSceneColor, 
				TranslucentBasePass.PrevSceneColorSampler,  
				SampleUV);

			SSR *= Vignette * saturate( 2 - 6.6 * Roughness );

			SSR.rgb *= TranslucentBasePass.PrevSceneColorPreExposureInv;
			SpecularIBL.rgb = SpecularIBL.rgb * (1 - SSR.a) + SSR.rgb;
		}
	}
#endif

	float3 SpecularLighting = SpecularIBL.rgb;

	// Have to opt-in to receiving planar reflections with forward shading
#if !FORWARD_SHADING || MATERIAL_PLANAR_FORWARD_REFLECTIONS
	// Plane normal will be zero if the feature is disabled
	BRANCH
	if (abs(dot(PlanarReflectionStruct.ReflectionPlane.xyz, 1)) > .0001f)
	{
		// Reuse ReflectionCubemapSampler to avoid reducing the sampler count available to artists
		float4 PlanarReflection = ComputePlanarReflections(MaterialParameters.WorldPosition_CamRelative, MaterialParameters.WorldNormal, Roughness);
		// Planar reflections win over SSR and reflection environment
		SpecularLighting = PlanarReflection.rgb + (1 - PlanarReflection.a) * SpecularLighting;
	}
#endif

	return SpecularLighting;
}

half3 GetImageBasedReflectionLighting(FMaterialPixelParameters MaterialParameters, half Roughness, half3 SpecularColor, half IndirectIrradiance, uint GridIndex, int SingleCaptureIndex)
{
	float3 N = MaterialParameters.WorldNormal;
	float3 V = MaterialParameters.CameraVector;

	float3 RayDirection = 2 * dot(V, N) * N - V;
	half NoV = saturate(dot(N, V));

	const float3 SpecularLighting = GetImageBasedReflectionSpecular(MaterialParameters, RayDirection, Roughness, IndirectIrradiance, GridIndex, SingleCaptureIndex);
		
#if MATERIAL_USE_PREINTEGRATED_GF
	SpecularColor = EnvBRDF(SpecularColor, Roughness, NoV);
#else
	SpecularColor = EnvBRDFApprox(SpecularColor, Roughness, NoV);
#endif
	return SpecularLighting * SpecularColor;
}
