// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ForwardShadowingCommon.ush
=============================================================================*/

#include "ShadowFilteringCommon.ush"

float ComputeDirectionalLightStaticShadowing(float3 TranslatedWorldPosition)
{
	float ShadowFactor = 1;
#if ALLOW_STATIC_LIGHTING
	BRANCH
	if (ForwardDirLightShadowStruct.DirectionalLightUseStaticShadowing > 0)
	{
		// Transform the world position into shadowmap space
		float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPosition, 1), ForwardDirLightShadowStruct.DirectionalLightTranslatedWorldToStaticShadow);
		float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;

		// Treat as unshadowed if the voxel is outside of the shadow map
		if (all(and(ShadowUVs >= 0, ShadowUVs <= 1)))
		{
			#define FILTER_STATIC_SHADOWING 0
			#if FILTER_STATIC_SHADOWING
				FPCFSamplerSettings Settings;
				Settings.ShadowDepthTexture = ForwardDirLightShadowStruct.DirectionalLightStaticShadowmap;
				Settings.ShadowDepthTextureSampler = ForwardDirLightShadowStruct.StaticShadowmapSampler;
				Settings.ShadowBufferSize = ForwardDirLightShadowStruct.DirectionalLightStaticShadowBufferSize;
				Settings.SceneDepth = 1 - HomogeneousShadowPosition.z;
				Settings.TransitionScale = 40;
				Settings.bSubsurface = false;
				// We can sample outside of the static shadowmap, which is centered around the lightmass importance volume.  These 'infinite' depth values should not cause occlusion.
				Settings.bTreatMaxDepthUnshadowed = true;
				Settings.DensityMulConstant = 0;
				Settings.ProjectionDepthBiasParameters = float2(0, 0);

				ShadowFactor = Manual1x1PCF(ShadowUVs, Settings);
			#else
				// Sample the shadowmap depth and determine if this voxel is shadowed
				float ShadowDepth = Texture2DSampleLevel(ForwardDirLightShadowStruct.DirectionalLightStaticShadowmap, ForwardDirLightShadowStruct.StaticShadowmapSampler, ShadowUVs, 0).x;
				ShadowFactor = HomogeneousShadowPosition.z < ShadowDepth || ShadowDepth > .99f;
			#endif
		}
	}
#endif
	return ShadowFactor;
}

#ifndef FILTER_DIRECTIONAL_LIGHT_SHADOWING
#define FILTER_DIRECTIONAL_LIGHT_SHADOWING 0
#endif

uint GetForwardLightingCascadeIndex(float4 CascadeEndDepths, float SceneDepth)
{
	float4 Count = float4(SceneDepth.xxxx >= CascadeEndDepths);
	return uint(Count.x + Count.y + Count.z + Count.w);
}

float ComputeDirectionalLightDynamicShadowing(float3 TranslatedWorldPosition, float SceneDepth, inout bool bShadowingFromValidUVArea)
{
	float ShadowFactor = 1;
	bShadowingFromValidUVArea = false;

	const uint NumCascades = ForwardDirLightShadowStruct.NumDirectionalLightCascades;
	if (NumCascades > 0)
	{
		uint CascadeIndex = GetForwardLightingCascadeIndex(ForwardDirLightShadowStruct.CascadeEndDepths, SceneDepth);
		if (CascadeIndex < NumCascades)
		{
			// Transform the world position into shadowmap space
			float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPosition, 1), ForwardDirLightShadowStruct.DirectionalLightTranslatedWorldToShadowMatrix[CascadeIndex]);
			float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;
			float4 ShadowmapMinMax = ForwardDirLightShadowStruct.DirectionalLightShadowmapMinMax[CascadeIndex];

			// Treat as unshadowed if the voxel is outside of the shadow map
			if (all(and(ShadowUVs >= ShadowmapMinMax.xy, ShadowUVs <= ShadowmapMinMax.zw)))
			{
			#if FILTER_DIRECTIONAL_LIGHT_SHADOWING
				FPCFSamplerSettings Settings;
				Settings.ShadowDepthTexture = ForwardDirLightShadowStruct.DirectionalLightShadowmapAtlas;
				Settings.ShadowDepthTextureSampler = ForwardDirLightShadowStruct.ShadowmapSampler;
				Settings.ShadowBufferSize = ForwardDirLightShadowStruct.DirectionalLightShadowmapAtlasBufferSize;
				Settings.SceneDepth = 1 - HomogeneousShadowPosition.z;
				Settings.TransitionScale = 4000;
				Settings.bSubsurface = false;
				Settings.bTreatMaxDepthUnshadowed = false;
				Settings.DensityMulConstant = 0;
				Settings.ProjectionDepthBiasParameters = float2(0, 0);

				ShadowFactor = Manual1x1PCF(ShadowUVs, Settings);
			#else
				// Sample the shadowmap depth and determine if this voxel is shadowed
				float ShadowDepth = Texture2DSampleLevel(ForwardDirLightShadowStruct.DirectionalLightShadowmapAtlas, ForwardDirLightShadowStruct.ShadowmapSampler, ShadowUVs, 0).x;
				ShadowFactor = 1 - HomogeneousShadowPosition.z < ShadowDepth - ForwardDirLightShadowStruct.DirectionalLightDepthBias.x;
			#endif

				// Note this doesn't actually mean the shadow factor computed is valid, eg CSM culling removes meshes which would cast on off-screen areas of the shadowmap, which still have valid UVs
				bShadowingFromValidUVArea = true;
			}
		}
	}

	return ShadowFactor;
}