// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FrontLayerTranslucency.usf
=============================================================================*/

#if defined(MainPS) && !OUTPUT_PIXEL_DEPTH_OFFSET
	// Force early depth/stencil test unless the material uses depth offset to speedup the front layer GBuffer pass.
	// Cant use EARLYDEPTHSTENCIL since that causes artifacts when combined with clip/discard.
	// COMPILER_FORCE_DEPTHSTENCIL_EARLYTEST_LATEWRITE must be defined before including Platform.ush
	#define COMPILER_FORCE_DEPTHSTENCIL_EARLYTEST_LATEWRITE 1
#endif // defined(MainPS) && !OUTPUT_PIXEL_DEPTH_OFFSET

#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "BRDF.ush"

#if defined(MainVS) || defined(MainPS)

// Material pass reroutes
#define SceneTexturesStruct FrontLayerTranslucencyGBufferPass.SceneTextures

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FFrontLayerTranslucencyGBufferInterpolantsVSToPS
{

};

struct FFrontLayerTranslucencyGBufferVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	FFrontLayerTranslucencyGBufferInterpolantsVSToPS PassInterpolants;
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
};

#endif // defined(MainVS) || defined(MainPS)

#ifdef MainVS

void MainVS(
	FVertexFactoryInput Input,
	out FFrontLayerTranslucencyGBufferVSToPS Output
	)
{	
	uint EyeIndex = 0;
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPositionExcludingWPO = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 WorldPosition = WorldPositionExcludingWPO;
	float4 ClipSpacePosition;

	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);

	ISOLATE
	{
		WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
		ClipSpacePosition = INVARIANT(mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip));
		Output.Position = INVARIANT(ClipSpacePosition);
	}

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
}

#endif // MainVS

#ifdef MainPS

#if !OUTPUT_PIXEL_DEPTH_OFFSET
	// Force early depth/stencil test unless the material uses depth offset to speedup the front layer GBuffer pass.
	// Cant use EARLYDEPTHSTENCIL since that causes artifacts when combined with clip/discard.
	// DEPTHSTENCIL_EARLYTEST_LATEWRITE can only be defined after including Platform.ush
	#define PIXELSHADER_EARLYDEPTHSTENCIL DEPTHSTENCIL_EARLYTEST_LATEWRITE
#else // OUTPUT_PIXEL_DEPTH_OFFSET
	#define PIXELSHADER_EARLYDEPTHSTENCIL
#endif // OUTPUT_PIXEL_DEPTH_OFFSET

PIXELSHADER_EARLYDEPTHSTENCIL
void MainPS(
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FFrontLayerTranslucencyGBufferInterpolantsVSToPS PassInterpolants,
	in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position		
	OPTIONAL_IsFrontFace,
	out float4 OutTarget0 : SV_Target0
#if PROJECT_SUPPORTS_MEGALIGHTS
	, out float4 OutTarget1 : SV_Target1
	, out float2 OutTarget2 : SV_Target2
#endif // PROJECT_SUPPORTS_MEGALIGHTS
	OPTIONAL_OutDepthConservative)
{
	ResolvedView = ResolveView();
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);

	FPixelMaterialInputs PixelMaterialInputs;
	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}
	float3 WorldNormal = MaterialParameters.WorldNormal;
	float Roughness = GetMaterialRoughness(PixelMaterialInputs);

	float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	float Opacity = GetMaterialOpacity(PixelMaterialInputs);
#else
	float Opacity = 1.0f;
#endif
	float Specular = GetMaterialSpecular(PixelMaterialInputs);
	float Metallic = GetMaterialMetallic(PixelMaterialInputs);
	uint LightingChannelMask = GetPrimitive_LightingChannelMask(MaterialParameters.PrimitiveId);

#if TEMPLATE_USES_SUBSTRATE
	{
		const float3 V = MaterialParameters.CameraVector;
		const float3 L = MaterialParameters.WorldNormal;

		// Initialise a Substrate header with normal in registers
		FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
		FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
		SubstratePixelHeader.IrradianceAO.MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);

		float TotalCoverage = 1.f;
		if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
		{
			const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, true /*bRoughDiffuseEnabled*/, 0 /*PeelLayersAboveDepth*/, false/*bRoughnessTracking*/);
			float3 TotalTransmittancePreCoverage = 0;
			SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V, Settings, TotalCoverage, TotalTransmittancePreCoverage);

			// Clip pixels that will never receive any lumen front material reflections.
			// We cannot generalise and use GetMaterialClippingVelocity for that because this function clips with a 1/255 threshold...that can result in NaN refelctions if some pixels are not genrated correctly.
			clip(TotalCoverage <= 0.0f ? -1.0f : 1.0f);

			// Extract averaged normal and roughness
			WorldNormal = 0;
			Roughness = 0;
			float TopLayerTotalWeight = 0.0f;
			Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
			{
				#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]
				if (SubstrateIsBSDFVisible(CurrentBSDF))
				{
					FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;	// Fake unused in SubstrateCreateBSDFContext when using Forward inline shading
					FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, V, L);
					const float Weight = Luminance(CurrentBSDF.LuminanceWeightV);
					WorldNormal += Weight * SubstrateBSDFContext.N;

					TopLayerTotalWeight += CurrentBSDF.TopLayerDataWeight;
					Roughness += CurrentBSDF.TopLayerDataWeight * SubstrateGetBSDFRoughness(CurrentBSDF);
					Opacity += CurrentBSDF.TopLayerDataWeight * SubstrateGetBSDFOpacity(CurrentBSDF);
					Metallic += CurrentBSDF.TopLayerDataWeight * SubstrateGetBSDFMetallic(CurrentBSDF);
					Specular += CurrentBSDF.TopLayerDataWeight * SubstrateGetBSDFSpecular(CurrentBSDF);
				}
				#undef CurrentBSDF
			}

			if (any(WorldNormal != 0)) 
			{
				WorldNormal = normalize(WorldNormal);
			}

			Roughness = TopLayerTotalWeight > 0.0f ? Roughness / TopLayerTotalWeight : 0.0f;
			Opacity = TopLayerTotalWeight > 0.0f ? Opacity / TopLayerTotalWeight : 0.0f;
			Metallic = TopLayerTotalWeight > 0.0f ? Metallic / TopLayerTotalWeight : 0.0f;
			Specular = TopLayerTotalWeight > 0.0f ? Specular / TopLayerTotalWeight : 0.0f;
		}
	}
#else
	// Only relevant to RT translucency. Force roughness to 0 for unlit to prevent running spatial reconstruction on these pixels
	if (GetMaterialShadingModel(PixelMaterialInputs) == SHADINGMODELID_UNLIT)
	{
		Roughness = 0.0f;
	}

	// Thin translucent materials are assumed to always have a coverage of 1.
#if !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	clip(PixelMaterialInputs.Opacity <= 0.0f ? -1.0f : 1.0f);
#endif
#endif // TEMPLATE_USES_SUBSTRATE

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

#if OUTPUT_PIXEL_DEPTH_OFFSET
	ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);
#endif

	// We store (1+roughness*65534)/65535 to make sure w is always greater that 0 where we write which then signifies we need lumen high quality reflections.
	// The render target is 16-bit unorm and we will still get enough accuracy.
	OutTarget0 = float4(EncodeNormal(WorldNormal), (1.0f + Roughness * 65534.0f) / 65535.0f);

#if PROJECT_SUPPORTS_MEGALIGHTS
#	if TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
	if (FrontLayerTranslucencyGBufferPass.OutputExtendedGBuffer > 0)
	{
		// 8:8:8 bit base color
		// 7 bit specular
		// 7 bit metallic
		// 7 bit opacity
		// 3 bit lighting channel

		const uint PackedOpacity = (uint)floor(Opacity * 127 + 0.5); // 7 bits
		const uint PackedSpecular = (uint)floor(Specular * 127 + 0.5); // 7 bits
		const uint PackedMetallic = (uint)floor(Metallic * 127 + 0.5); // 7 bits

		uint PackedW = PackedOpacity | (((LightingChannelMask >> 0) & 0x1) << 7);
		PackedW = max(PackedW, 1u); // make sure w is always greater that 0 where we write which then signifies we need MegaLights high quality lighting.
		float PackedWFloat = PackedW / 255.0f;

		uint PackedR = PackedSpecular | (((LightingChannelMask >> 1) & 0x1) << 7);
		float PackedRFloat = PackedR / 255.0f;

		uint PackedG = PackedMetallic | (((LightingChannelMask >> 2) & 0x1) << 7);
		float PackedGFloat = PackedG / 255.0f;

		OutTarget1 = float4(BaseColor, PackedWFloat);
		OutTarget2 = float2(PackedRFloat, PackedGFloat);

		// Mark this tile as containing a valid surface for MegaLights front layer translucency (equivalent to Material.IsMegaLightsValid() == true).
		// Multiple pixels in the same tile all write 1, so no atomics are needed.
		FrontLayerTranslucencyGBufferPass.RWMegaLightsTileMask[(uint2)SvPosition.xy / FRONT_LAYER_TILE_SIZE] = 1u;
	}
	else
#	endif // TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
	{
		OutTarget1 = 0.0f;
		OutTarget2 = 0.0f;
	}
#endif // PROJECT_SUPPORTS_MEGALIGHTS
}

#endif // MainPS

#ifdef FrontLayerTranslucencyClearGBufferPS

void FrontLayerTranslucencyClearGBufferPS(
	  in  float4 SvPosition		: SV_Position
	, out float4 OutColor0		: SV_Target0
#if PROJECT_SUPPORTS_MEGALIGHTS
	, out float4 OutColor1		: SV_Target1
#endif // PROJECT_SUPPORTS_MEGALIGHTS
	, out float  OutDepth		: SV_DEPTH)
{
	const uint2 PixelCoord = uint2(SvPosition.xy);

	OutColor0 = 0;
#if PROJECT_SUPPORTS_MEGALIGHTS
	OutColor1 = 0;
#endif // PROJECT_SUPPORTS_MEGALIGHTS
	OutDepth = SceneTexturesStruct.SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
}

#endif // FrontLayerTranslucencyClearGBufferPS

#ifdef BuildFrontLayerTileListCS

Texture2D<uint> FrontLayerTileMask;
RWStructuredBuffer<uint> RWTileAllocator;
RWStructuredBuffer<uint> RWTileData;
int2 TileViewRectMin;
int2 TileViewRectSize;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void BuildFrontLayerTileListCS(uint2 LocalTileCoord : SV_DispatchThreadID)
{
	if (any(int2(LocalTileCoord) >= TileViewRectSize))
	{
		return;
	}

	const uint2 TileCoord = uint2(int2(LocalTileCoord) + TileViewRectMin);

	if (FrontLayerTileMask[TileCoord] != 0u)
	{
		uint Index;
		InterlockedAdd(RWTileAllocator[0], 1u, Index);

		// Store view-relative tile coords.
		RWTileData[Index] = LocalTileCoord.x | (LocalTileCoord.y << 16u);
	}
}

#endif // BuildFrontLayerTileListCS

#ifdef InitFrontLayerTileIndirectArgsCS

StructuredBuffer<uint> TileAllocator;
RWBuffer<uint> RWTileIndirectArgs;

[numthreads(1, 1, 1)]
void InitFrontLayerTileIndirectArgsCS()
{
	WriteDispatchIndirectArgs(RWTileIndirectArgs, 0, TileAllocator[0], 1, 1);
}

#endif // InitFrontLayerTileIndirectArgsCS
