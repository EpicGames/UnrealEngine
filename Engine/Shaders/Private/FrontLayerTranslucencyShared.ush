// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	FrontLayerTranslucencyShared.ush
=============================================================================*/

#include "SHCommon.ush"

// Temp version GUID to force rebuild of shaders using the FFrontLayerTranslucencyParameters
// uniform buffer, until versioning of uniform buffer structs works properly.
#pragma message("UESHADERMETADATA_VERSION F35DABF2-F5F7-4F7C-9056-AF53F2A1DB60")

bool UseFrontLayerReflection(float2 BufferUV, float SceneDepth, bool bIsTranslucencyAfterMotionBlur)
{
	if (bIsTranslucencyAfterMotionBlur)
	{
		// For translucent rendered after motion blur, we cannot use front layer lighting because depth test is disabled in this case 
		// (depth is jittered and post motion blur composition can happen at a different resolution with dynamic resolution scaling).
		// Such shading would look different on translucent meshes when occluded by opaque meshes.
		return false;
	}

	bool bValid = false;

	#if TRANSLUCENT_WRITING_FRONT_LAYER_TRANSPARENCY
	if (FrontLayerTranslucencyStruct.ReflectionEnabled > 0)
	{
		float SingleLayerDeviceZ = Texture2DSampleLevel(FrontLayerTranslucencyStruct.SceneDepth, GlobalPointClampedSampler, BufferUV, 0.0f).x;
		float DeviceZ = ConvertToDeviceZ(SceneDepth);

		// ULP float comparison
		if (abs(asint(SingleLayerDeviceZ) - asint(DeviceZ)) <= FrontLayerTranslucencyStruct.RelativeDepthThreshold)
		{
			bValid = true;
		}
	}
	#endif

	return bValid;
}

float3 SampleFrontLayerReflection(float2 BufferUV)
{
	const float3 Radiance = Texture2DArraySample(FrontLayerTranslucencyStruct.Reflection, GlobalPointClampedSampler, float3(BufferUV, 0.0f)).xyz * View.OneOverPreExposure;
	
	// Constrast adjustment with mid-grey as 'anchor' point so that overall exposure don't change
	return (pow(max(0.f, Radiance * (1.f / 0.18f)), FrontLayerTranslucencyStruct.Contrast) * 0.18f) * FrontLayerTranslucencyStruct.SpecularScale;
}

bool UseFrontLayerDirectLighting(float2 BufferUV, float SceneDepth, bool bIsTranslucencyAfterMotionBlur)
{
	if (bIsTranslucencyAfterMotionBlur)
	{
		// For translucent rendered after motion blur, we cannot use front layer lighting because depth test is disabled in this case 
		// (depth is jittered and post motion blur composition can happen at a different resolution with dynamic resolution scaling).
		// Such shading would look different on translucent meshes when occluded by opaque meshes.
		return false;
	}

	bool bValid = false;

#if TRANSLUCENT_WRITING_FRONT_LAYER_TRANSPARENCY
	if (FrontLayerTranslucencyStruct.DirectLightingEnabled > 0)
	{
		float SingleLayerDeviceZ = Texture2DSampleLevel(FrontLayerTranslucencyStruct.SceneDepth, GlobalPointClampedSampler, BufferUV, 0.0f).x;
		float DeviceZ = ConvertToDeviceZ(SceneDepth);

		// ULP float comparison
		if (abs(asint(SingleLayerDeviceZ) - asint(DeviceZ)) <= FrontLayerTranslucencyStruct.RelativeDepthThreshold)
		{
			bValid = true;
		}
	}
#endif

	return bValid;
}

float3 SampleFrontLayerDirectLighting(float2 BufferUV)
{
	return Texture2DSample(FrontLayerTranslucencyStruct.DirectLighting, GlobalPointClampedSampler, BufferUV).xyz * View.OneOverPreExposure;
}
