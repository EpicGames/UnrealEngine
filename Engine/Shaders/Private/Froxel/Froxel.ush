// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Shared/FroxelDefinitions.h"

// Froxel grid constants
float4 FroxelToViewScaleBias;
float4 FroxelToClipScaleBias;
float2 FroxelClipToViewScale;
float FroxelRecLog2DepthScale1; // 1 / log2(1.0 + DepthScale)
float FroxelRecNearScale;
float FroxelDepthScale1;
float FroxelNear;
float FroxelViewToClipTransformScale;
float FroxelClipToViewTransformScale;
float FroxelClipToViewTransformBias;
float FroxelRadius;
float FroxelInvRadius;

uint bFroxelIsOrtho;
uint FroxelArgsStride;
uint FroxelArgsOffset;

int CalcFroxelSliceFromDeviceZ(float DeviceZ)
{
	float DeviceZScaleBias = DeviceZ * FroxelClipToViewTransformScale + FroxelClipToViewTransformBias;

	if (bFroxelIsOrtho)
	{
		int K = int(floor((DeviceZScaleBias - FroxelNear) * FroxelInvRadius));
		return K;
	}

	// Special case for infinity
	if (DeviceZ == 0.0f)
	{
		return FROXEL_INVALID_SLICE;
	}

	int K = int(floor(log2(rcp(DeviceZScaleBias) * FroxelRecNearScale) * FroxelRecLog2DepthScale1));
	return K;
}

float GetViewDepthFromFroxelSlice(float SliceZ)
{
	if (bFroxelIsOrtho)
	{
		return SliceZ * FroxelRadius + FroxelNear;
	}
	return FroxelNear * pow(FroxelDepthScale1, SliceZ);
}

float GetViewDepthFromFroxelSlice(int SliceZ)
{
	return GetViewDepthFromFroxelSlice(float(SliceZ));
}

float GetClipDepthFromFroxelSlice(float SliceZ)
{
	float ViewDepth = GetViewDepthFromFroxelSlice(SliceZ);

	float TmpViewDepth = ViewDepth - FroxelClipToViewTransformBias;
	if (!bFroxelIsOrtho)
	{
		TmpViewDepth = rcp(TmpViewDepth);
	}
	return TmpViewDepth * FroxelViewToClipTransformScale;
}

FPackedFroxel PackFroxel(int3 Froxel)
{
	FPackedFroxel Result;
	Result.XY = uint(Froxel.x) | (uint(Froxel.y) << 16u);
	Result.Z = Froxel.z;
	return Result;
}

int3 UnpackFroxel(FPackedFroxel PackedFroxel)
{
	int3 Result;
	Result.x = int(BitFieldExtractU32(PackedFroxel.XY, 16, 0));
	Result.y = int(BitFieldExtractU32(PackedFroxel.XY, 16, 16));
	Result.z = PackedFroxel.Z;

	return Result;
}

float3 GetFroxelViewSpaceCenter(int3 Froxel)
{
	float ViewSpaceCenterZ = GetViewDepthFromFroxelSlice(float(Froxel.z) + 0.5f);
	float2 TmpXY = (float2(Froxel.xy) + 0.5f) * FroxelToViewScaleBias.xy + FroxelToViewScaleBias.zw;
	if (bFroxelIsOrtho)
	{
		return float3(TmpXY, ViewSpaceCenterZ);
	}
	return float3(TmpXY * ViewSpaceCenterZ, ViewSpaceCenterZ);
}

struct FFroxelViewBounds
{
	float3 Min;
	float3 Max;
};

FFroxelViewBounds GetFroxelViewSpaceAABB(int3 Froxel)
{
	float2 MinXY = ((float2(Froxel.xy)) * FroxelToViewScaleBias.xy + FroxelToViewScaleBias.zw);
	float2 MaxXY = MinXY + FroxelToViewScaleBias.xy;

	float MinViewZ = GetViewDepthFromFroxelSlice(Froxel.z);
	FFroxelViewBounds Result;
	if (bFroxelIsOrtho)
	{
		Result.Min.xy = MinXY;
		Result.Min.z = MinViewZ;
		Result.Max.xy = MaxXY;
		Result.Max.z = MinViewZ + FroxelRadius; 
		return Result;
	}

	if (abs(MinXY.x) > abs(MaxXY.x))
	{
		Swap(MinXY.x, MaxXY.x);
	}
	if (abs(MinXY.y) > abs(MaxXY.y))
	{
		Swap(MinXY.y, MaxXY.y);
	}

	float MaxViewZ = MinViewZ * FroxelDepthScale1;

	// Extrema corners in X plane near and far
	float2 ExtNear = MinXY * MinViewZ;
	float2 ExtFar = MaxXY * MaxViewZ; // == 

	Result.Min.xy = min(ExtNear, ExtFar);
	Result.Min.z = MinViewZ;
	Result.Max.xy = max(ExtNear, ExtFar);
	Result.Max.z = MaxViewZ; 

	return Result;
}

struct FFroxelClipBounds
{
	float3 ClipMin;
	float3 ClipMax;
};

FFroxelClipBounds GetFroxelClipBounds(int3 Froxel)
{
	float MinDeviceZ = GetClipDepthFromFroxelSlice(Froxel.z);
	float MaxDeviceZ = GetClipDepthFromFroxelSlice(Froxel.z + 1.0f);

	float2 MinClipSpaceXY = ((float2(Froxel.xy)) * FroxelToClipScaleBias.xy + FroxelToClipScaleBias.zw);
	float2 MaxClipSpaceXY = MinClipSpaceXY + FroxelToClipScaleBias.xy;

	FFroxelClipBounds Result;
	Result.ClipMin = float3(MinClipSpaceXY, MinDeviceZ);
	Result.ClipMax = float3(MaxClipSpaceXY, MaxDeviceZ);
	
	return Result;
}