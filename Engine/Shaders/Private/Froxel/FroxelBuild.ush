// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "Froxel.ush"
#include "../WaveOpUtil.ush"

#if !defined(FROXEL_HASH_BUFFER_SIZE) || !defined(FROXEL_SHARED_HASH_BUFFER_VAR) || !defined(FROXEL_HASH_THREAD_GROUP_SIZE)
#error "Must define FROXEL_HASH_BUFFER_SIZE and FROXEL_SHARED_HASH_BUFFER_VAR and FROXEL_HASH_THREAD_GROUP_SIZE to use hash froxel builder"
#endif

#if FROXEL_HASH_THREAD_GROUP_SIZE != FROXEL_HASH_BUFFER_SIZE
#error "FROXEL_HASH_THREAD_GROUP_SIZE and FROXEL_HASH_BUFFER_SIZE must be the same size"
#endif

RWStructuredBuffer<FPackedFroxel> OutFroxels;
RWBuffer<uint> OutFroxelArgs;

groupshared int GroupShared_FroxelAnyTrue;

bool GroupActiveAnyTrue(bool bValue)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	bool bWaveValue = WaveActiveAnyTrue(bValue);
	const uint LaneCount = WaveGetLaneCount();
	// enough to just use the vote, skip further reductions
	if (LaneCount >= FROXEL_HASH_THREAD_GROUP_SIZE)
	{
		return bWaveValue;
	}
	// no need to write to group shared from any other lanes.
	if (!WaveIsFirstLane())
	{
		bWaveValue = false;
	}
#else
	bool bWaveValue = bValue;
#endif
	
	// use group shared variable to broadcast result.
	GroupShared_FroxelAnyTrue = 0;
	GroupMemoryBarrierWithGroupSync();
	if (bWaveValue)
	{
		GroupShared_FroxelAnyTrue = 1;
	}
	GroupMemoryBarrierWithGroupSync();
	bool bResult = GroupShared_FroxelAnyTrue != 0;
	GroupMemoryBarrierWithGroupSync();
	return bResult;
}

void EmitFroxel(int3 Froxel)
{
	// write directly to global buffer
	uint WriteOffset = 0;
	WaveInterlockedAddScalarInGroups(OutFroxelArgs[FroxelArgsOffset * FroxelArgsStride + 3], OutFroxelArgs[FroxelArgsOffset * FroxelArgsStride], 64, 1, WriteOffset);
	OutFroxels[WriteOffset] = PackFroxel(Froxel);
}

void EmitValidFroxel(int3 Froxel)
{
	if (Froxel.z != FROXEL_INVALID_SLICE)
	{
		EmitFroxel(Froxel);
	}
}

void WriteHashedSlice(uint PackedSliceId)
{
	if (PackedSliceId < FROXEL_INVALID_PACKED_SLICE)
	{
		FROXEL_SHARED_HASH_BUFFER_VAR[PackedSliceId % FROXEL_HASH_BUFFER_SIZE] = PackedSliceId;
	}
}

void CheckHashedSlice(inout uint PackedSliceId, inout bool bAnyValidLeft)
{
	if (PackedSliceId < FROXEL_INVALID_PACKED_SLICE)
	{
		if (FROXEL_SHARED_HASH_BUFFER_VAR[PackedSliceId % FROXEL_HASH_BUFFER_SIZE] == PackedSliceId)
		{
			PackedSliceId = FROXEL_INVALID_PACKED_SLICE;
		}
	}
	if (PackedSliceId < FROXEL_INVALID_PACKED_SLICE)
	{
		bAnyValidLeft = true;
	}
}

groupshared int GroupShared_FroxelWorkLeft;
void HashReduceFroxels(int MaxValid, uint GroupThreadIndex, uint LocalLinearTileId, uint2 GroupTileOffset, uint4 SortedSlices)
{
	// pack the slice IDs
	SortedSlices.x =	 (uint(SortedSlices.x) << 2u) | LocalLinearTileId;
	if (MaxValid > 1)
	{
		SortedSlices.y = (uint(SortedSlices.y) << 2u) | LocalLinearTileId;
	}
	if (MaxValid > 2)
	{
		SortedSlices.z = (uint(SortedSlices.z) << 2u) | LocalLinearTileId;
	}
	if (MaxValid > 3)
	{
		SortedSlices.w = (uint(SortedSlices.w) << 2u) | LocalLinearTileId;
	}
	
	LOOP
	for (int Iter = 0; Iter < 4; ++Iter)
	{
		FROXEL_SHARED_HASH_BUFFER_VAR[GroupThreadIndex] = FROXEL_INVALID_PACKED_SLICE;
		GroupMemoryBarrierWithGroupSync();
		GroupShared_FroxelWorkLeft = 0;

		// Insert the unique depth slices into the hash. We do all at once because if they don't collide we can reduce the number of barriers and do more in each iteration.
		WriteHashedSlice(SortedSlices.x);
		if (MaxValid > 1)
		{
			WriteHashedSlice(SortedSlices.y);
		}
		if (MaxValid > 2)
		{
			WriteHashedSlice(SortedSlices.z);
		}
		if (MaxValid > 3)
		{
			WriteHashedSlice(SortedSlices.w);
		}
		GroupMemoryBarrierWithGroupSync();

		// 2. Look up and clear own ID
		bool bAnyValidLeft = false;
		CheckHashedSlice(SortedSlices.x, bAnyValidLeft);
		if (MaxValid > 1)
		{
			CheckHashedSlice(SortedSlices.y, bAnyValidLeft);
		}
		if (MaxValid > 2)
		{
			CheckHashedSlice(SortedSlices.z, bAnyValidLeft);
		}
		if (MaxValid > 3)
		{
			CheckHashedSlice(SortedSlices.w, bAnyValidLeft);
		}
		// Load thread coherently and emit
		uint PackedId = FROXEL_SHARED_HASH_BUFFER_VAR[GroupThreadIndex];
		if (PackedId < FROXEL_INVALID_PACKED_SLICE)
		{
			uint2 FinishedLocalTileId = uint2(PackedId & 1u, (PackedId & 2u) >> 1u);
			int3 FinishedFroxel = int3(GroupTileOffset + FinishedLocalTileId, int(PackedId >> 2u) - int(FROXEL_PACKED_SLICE_BIAS));

			EmitFroxel(FinishedFroxel);
		}

		if (GroupActiveAnyTrue(bAnyValidLeft))
		{
			break;
		}
	}
}

void CompSwap(inout uint A, inout uint B)
{
	if (A > B)
	{
		Swap(A, B);
	}
}

void HashBuildFroxelsFromDeviceZ(float4 DeviceZ, uint GroupThreadIndex, uint LocalLinearTileId, uint2 GroupTileOffset)
{
	int4 Slice2x2;
	Slice2x2.x = CalcFroxelSliceFromDeviceZ(DeviceZ.x);
	Slice2x2.y = CalcFroxelSliceFromDeviceZ(DeviceZ.y);
	Slice2x2.z = CalcFroxelSliceFromDeviceZ(DeviceZ.z);
	Slice2x2.w = CalcFroxelSliceFromDeviceZ(DeviceZ.w);

	uint4 SortedSlices = uint4(Slice2x2 + FROXEL_PACKED_SLICE_BIAS);

	// Sort 
	CompSwap(SortedSlices.x, SortedSlices.z);
	CompSwap(SortedSlices.y, SortedSlices.w);
	CompSwap(SortedSlices.x, SortedSlices.y);
	CompSwap(SortedSlices.z, SortedSlices.w);
	CompSwap(SortedSlices.y, SortedSlices.z);

	// make invalid those that are the same as neighbor (must go in descending order)
	SortedSlices.w = select(SortedSlices.w != SortedSlices.z, SortedSlices.w, FROXEL_INVALID_SLICE);
	SortedSlices.z = select(SortedSlices.z != SortedSlices.y, SortedSlices.z, FROXEL_INVALID_SLICE);
	SortedSlices.y = select(SortedSlices.y != SortedSlices.x, SortedSlices.y, FROXEL_INVALID_SLICE);

	// re-sort the three last ones (x does not change as per the above)
	CompSwap(SortedSlices.y, SortedSlices.w);
	CompSwap(SortedSlices.y, SortedSlices.z);
	CompSwap(SortedSlices.z, SortedSlices.w);

	// Skip to wave-optimized version for remaining number of items per thread.
    if (GroupActiveAnyTrue(SortedSlices.w < FROXEL_INVALID_SLICE))
	{
		HashReduceFroxels(4, GroupThreadIndex, LocalLinearTileId, GroupTileOffset, SortedSlices);
	}
	// TODO: Wave vote not safe since we need whole wave active in the reduction phase, need full group, so only safe if group size == wave size.
	else if (GroupActiveAnyTrue(SortedSlices.z < FROXEL_INVALID_SLICE))
	{
		HashReduceFroxels(3, GroupThreadIndex, LocalLinearTileId, GroupTileOffset, SortedSlices);
	}
	else if (GroupActiveAnyTrue(SortedSlices.y < FROXEL_INVALID_SLICE))
	{
		HashReduceFroxels(2, GroupThreadIndex, LocalLinearTileId, GroupTileOffset, SortedSlices);
	}
	else if (GroupActiveAnyTrue(SortedSlices.x < FROXEL_INVALID_SLICE))
	{
		HashReduceFroxels(1, GroupThreadIndex, LocalLinearTileId, GroupTileOffset, SortedSlices);
	}
}

void HashBuildFroxelsFromDeviceZ(float DeviceZ, uint GroupThreadIndex, uint LocalLinearTileId, uint2 GroupTileOffset)
{
	int4 SortedSlices;
	// Sorted by definition so no need to sort
	SortedSlices.x = CalcFroxelSliceFromDeviceZ(DeviceZ) + FROXEL_PACKED_SLICE_BIAS;
	SortedSlices.y = SortedSlices.z = SortedSlices.w = FROXEL_INVALID_SLICE;
	// TODO: Wave vote not safe since we need whole wave active in the reduction phase, need full group, so only safe if group size == wave size.
    if (GroupActiveAnyTrue(SortedSlices.x < FROXEL_INVALID_SLICE))
	{
		HashReduceFroxels(1, GroupThreadIndex, LocalLinearTileId, GroupTileOffset, SortedSlices);
	}
}
