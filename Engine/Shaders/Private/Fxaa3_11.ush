/*============================================================================


                    NVIDIA FXAA 3.11 by TIMOTHY LOTTES


------------------------------------------------------------------------------
COPYRIGHT (C) 2010, 2011 NVIDIA CORPORATION. ALL RIGHTS RESERVED.
------------------------------------------------------------------------------
TO THE MAXIMUM EXTENT PERMITTED BY APPLICABLE LAW, THIS SOFTWARE IS PROVIDED
*AS IS* AND NVIDIA AND ITS SUPPLIERS DISCLAIM ALL WARRANTIES, EITHER EXPRESS
OR IMPLIED, INCLUDING, BUT NOT LIMITED TO, IMPLIED WARRANTIES OF
MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE. IN NO EVENT SHALL NVIDIA
OR ITS SUPPLIERS BE LIABLE FOR ANY SPECIAL, INCIDENTAL, INDIRECT, OR
CONSEQUENTIAL DAMAGES WHATSOEVER (INCLUDING, WITHOUT LIMITATION, DAMAGES FOR
LOSS OF BUSINESS PROFITS, BUSINESS INTERRUPTION, LOSS OF BUSINESS INFORMATION,
OR ANY OTHER PECUNIARY LOSS) ARISING OUT OF THE USE OF OR INABILITY TO USE
THIS SOFTWARE, EVEN IF NVIDIA HAS BEEN ADVISED OF THE POSSIBILITY OF SUCH
DAMAGES.

------------------------------------------------------------------------------
                           INTEGRATION CHECKLIST
------------------------------------------------------------------------------
(1.)
In the shader source, setup defines for the desired configuration.
When providing multiple shaders (for different presets),
simply setup the defines differently in multiple files.
Example,

  #define FXAA_PC 1
  #define FXAA_HLSL_5 1
  #define FXAA_QUALITY__PRESET 12

Or,

  #define FXAA_360 1
  
Or,

  #define FXAA_PS3 1
  
Etc.

(2.)
Then include this file using the include directive,


"Fxaa3_11.h"

(3.)
Then call the FXAA pixel shader from within your desired shader.
Look at the FXAA Quality FxaaPixelShader() for docs on inputs.
As for FXAA 3.11 all inputs for all shaders are the same 
to enable easy porting between platforms.

  return FxaaPixelShader(...);

(4.)
Insure pass prior to FXAA outputs RGBL (see next section).
Or use,

  #define FXAA_GREEN_AS_LUMA 1

(5.)
Setup engine to provide the following constants
which are used in the FxaaPixelShader() inputs,

  FxaaFloat2 fxaaQualityRcpFrame,
  FxaaFloat4 fxaaConsoleRcpFrameOpt,
  FxaaFloat4 fxaaConsoleRcpFrameOpt2,
  FxaaFloat4 fxaaConsole360RcpFrameOpt2,
  FxaaFloat fxaaQualitySubpix,
  FxaaFloat fxaaQualityEdgeThreshold,
  FxaaFloat fxaaQualityEdgeThresholdMin,
  FxaaFloat fxaaConsoleEdgeSharpness,
  FxaaFloat fxaaConsoleEdgeThreshold,
  FxaaFloat fxaaConsoleEdgeThresholdMin,
  FxaaFloat4 fxaaConsole360ConstDir

Look at the FXAA Quality FxaaPixelShader() for docs on inputs.

(6.)
Have FXAA vertex shader run as a full screen triangle,
and output "pos" and "fxaaConsolePosPos" 
such that inputs in the pixel shader provide,

  // {xy} = center of pixel
  FxaaFloat2 pos,

  // {xy__} = upper left of pixel
  // {__zw} = lower right of pixel
  FxaaFloat4 fxaaConsolePosPos,

(7.)
Insure the texture sampler(s) used by FXAA are set to bilinear filtering.


------------------------------------------------------------------------------
                    INTEGRATION - RGBL AND COLORSPACE
------------------------------------------------------------------------------
FXAA3 requires RGBL as input unless the following is set, 

  #define FXAA_GREEN_AS_LUMA 1

In which case the engine uses green in place of luma,
and requires RGB input is in a non-linear colorspace.

RGB should be LDR (low dynamic range).
Specifically do FXAA after tonemapping.

RGB data as returned by a texture fetch can be non-linear,
or linear when FXAA_GREEN_AS_LUMA is not set.
Note an "sRGB format" texture counts as linear,
because the result of a texture fetch is linear data.
Regular "RGBA8" textures in the sRGB colorspace are non-linear.

If FXAA_GREEN_AS_LUMA is not set,
luma must be stored in the alpha channel prior to running FXAA.
This luma should be in a perceptual space (could be gamma 2.0).
Example pass before FXAA where output is gamma 2.0 encoded,

  color.rgb = ToneMap(color.rgb); // linear color output
  color.rgb = sqrt(color.rgb);    // gamma 2.0 color output
  return color;

To use FXAA,

  color.rgb = ToneMap(color.rgb);  // linear color output
  color.rgb = sqrt(color.rgb);     // gamma 2.0 color output
  color.a = dot(color.rgb, FxaaFloat3(0.299, 0.587, 0.114)); // compute luma
  return color;

Another example where output is linear encoded,
say for instance writing to an sRGB formated render target,
where the render target does the conversion back to sRGB after blending,

  color.rgb = ToneMap(color.rgb); // linear color output
  return color;

To use FXAA,

  color.rgb = ToneMap(color.rgb); // linear color output
  color.a = sqrt(dot(color.rgb, FxaaFloat3(0.299, 0.587, 0.114))); // compute luma
  return color;

Getting luma correct is required for the algorithm to work correctly.


------------------------------------------------------------------------------
                          BEING LINEARLY CORRECT?
------------------------------------------------------------------------------
Applying FXAA to a framebuffer with linear RGB color will look worse.
This is very counter intuitive, but happends to be true in this case.
The reason is because dithering artifacts will be more visible 
in a linear colorspace.


------------------------------------------------------------------------------
                             COMPLEX INTEGRATION
------------------------------------------------------------------------------
Q. What if the engine is blending into RGB before wanting to run FXAA?

A. In the last opaque pass prior to FXAA,
   have the pass write out luma into alpha.
   Then blend into RGB only.
   FXAA should be able to run ok
   assuming the blending pass did not any add aliasing.
   This should be the common case for particles and common blending passes.

A. Or use FXAA_GREEN_AS_LUMA.

============================================================================*/

/*============================================================================

                             INTEGRATION KNOBS

============================================================================*/
//
// FXAA_PS3 and FXAA_360 choose the console algorithm (FXAA3 CONSOLE).
// FXAA_360_OPT is a prototype for the new optimized 360 version.
//
// 1 = Use API.
// 0 = Don't use API.
//
/*--------------------------------------------------------------------------*/
#ifndef FXAA_PS3
    #define FXAA_PS3 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_360
    #define FXAA_360 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_360_OPT
    #define FXAA_360_OPT 0
#endif
/*==========================================================================*/
#ifndef FXAA_PC
    //
    // FXAA Quality
    // The high quality PC algorithm.
    //
    #define FXAA_PC 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_PC_CONSOLE
    //
    // The console algorithm for PC is included
    // for developers targeting really low spec machines.
    // Likely better to just run FXAA_PC, and use a really low preset.
    //
    #define FXAA_PC_CONSOLE 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_GLSL_120
    #define FXAA_GLSL_120 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_GLSL_130
    #define FXAA_GLSL_130 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_HLSL_3
    #define FXAA_HLSL_3 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_HLSL_4
    #define FXAA_HLSL_4 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_HLSL_5
    #define FXAA_HLSL_5 0
#endif
/*==========================================================================*/
#ifndef FXAA_GREEN_AS_LUMA
    //
    // For those using non-linear color,
    // and either not able to get luma in alpha, or not wanting to,
    // this enables FXAA to run using green as a proxy for luma.
    // So with this enabled, no need to pack luma in alpha.
    //
    // This will turn off AA on anything which lacks some amount of green.
    // Pure red and blue or combination of only R and B, will get no AA.
    //
    // Might want to lower the settings for both,
    //    fxaaConsoleEdgeThresholdMin
    //    fxaaQualityEdgeThresholdMin
    // In order to insure AA does not get turned off on colors 
    // which contain a minor amount of green.
    //
    // 1 = On.
    // 0 = Off.
    //
    #define FXAA_GREEN_AS_LUMA 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_EARLY_EXIT
    //
    // Controls algorithm's early exit path.
    // On PS3 turning this ON adds 2 cycles to the shader.
    // On 360 turning this OFF adds 10ths of a millisecond to the shader.
    // Turning this off on console will result in a more blurry image.
    // So this defaults to on.
    //
    // 1 = On.
    // 0 = Off.
    //
    #define FXAA_EARLY_EXIT 1
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_DISCARD
    //
    // Only valid for PC OpenGL currently.
    // Probably will not work when FXAA_GREEN_AS_LUMA = 1.
    //
    // 1 = Use discard on pixels which don't need AA.
    //     For APIs which enable concurrent TEX+ROP from same surface.
    // 0 = Return unchanged color on pixels which don't need AA.
    //
    #define FXAA_DISCARD 0
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_FAST_PIXEL_OFFSET
    //
    // Used for GLSL 120 only.
    //
    // 1 = GL API supports fast pixel offsets
    // 0 = do not use fast pixel offsets
    //
    #ifdef GL_EXT_gpu_shader4
        #define FXAA_FAST_PIXEL_OFFSET 1
    #endif
    #ifdef GL_NV_gpu_shader5
        #define FXAA_FAST_PIXEL_OFFSET 1
    #endif
    #ifdef GL_ARB_gpu_shader5
        #define FXAA_FAST_PIXEL_OFFSET 1
    #endif
    #ifndef FXAA_FAST_PIXEL_OFFSET
        #define FXAA_FAST_PIXEL_OFFSET 0
    #endif
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_GATHER4_ALPHA
    //
    // 1 = API supports gather4 on alpha channel.
    // 0 = API does not support gather4 on alpha channel.
    //
    //#if (FXAA_HLSL_5 == 1)
    //    #define FXAA_GATHER4_ALPHA 1
    //#endif
    //#ifdef GL_ARB_gpu_shader5
    //    #define FXAA_GATHER4_ALPHA 1
    //#endif
    //#ifdef GL_NV_gpu_shader5
    //    #define FXAA_GATHER4_ALPHA 1
    //#endif
    //#ifndef FXAA_GATHER4_ALPHA
    //    #define FXAA_GATHER4_ALPHA 0
    //#endif
	#define FXAA_GATHER4_ALPHA 0
#endif

/*============================================================================
                      FXAA CONSOLE PS3 - TUNING KNOBS
============================================================================*/
#ifndef FXAA_CONSOLE__PS3_EDGE_SHARPNESS
    //
    // Consoles the sharpness of edges on PS3 only.
    // Non-PS3 tuning is done with shader input.
    //
    // Due to the PS3 being ALU bound,
    // there are only two safe values here: 4 and 8.
    // These options use the shaders ability to a free *|/ by 2|4|8.
    //
    // 8.0 is sharper
    // 4.0 is softer
    // 2.0 is really soft (good for vector graphics inputs)
    //
    #if 1
        #define FXAA_CONSOLE__PS3_EDGE_SHARPNESS 8.0
    #endif
    #if 0
        #define FXAA_CONSOLE__PS3_EDGE_SHARPNESS 4.0
    #endif
    #if 0
        #define FXAA_CONSOLE__PS3_EDGE_SHARPNESS 2.0
    #endif
#endif
/*--------------------------------------------------------------------------*/
#ifndef FXAA_CONSOLE__PS3_EDGE_THRESHOLD
    //
    // Only effects PS3.
    // Non-PS3 tuning is done with shader input.
    //
    // The minimum amount of local contrast required to apply algorithm.
    // The console setting has a different mapping than the quality setting.
    //
    // This only applies when FXAA_EARLY_EXIT is 1.
    //
    // Due to the PS3 being ALU bound,
    // there are only two safe values here: 0.25 and 0.125.
    // These options use the shaders ability to a free *|/ by 2|4|8.
    //
    // 0.125 leaves less aliasing, but is softer
    // 0.25 leaves more aliasing, and is sharper
    //
    #if 1
        #define FXAA_CONSOLE__PS3_EDGE_THRESHOLD 0.125
    #else
        #define FXAA_CONSOLE__PS3_EDGE_THRESHOLD 0.25
    #endif
#endif

/*============================================================================
                        FXAA QUALITY - TUNING KNOBS
------------------------------------------------------------------------------
NOTE the other tuning knobs are now in the shader function inputs!
============================================================================*/
#ifndef FXAA_QUALITY__PRESET
    //
    // Choose the quality preset.
    // This needs to be compiled into the shader as it effects code.
    // Best option to include multiple presets is to 
    // in each shader define the preset, then include this file.
    // 
    // OPTIONS
    // -----------------------------------------------------------------------
    // 10 to 15 - default medium dither (10=fastest, 15=highest quality)
    // 20 to 29 - less dither, more expensive (20=fastest, 29=highest quality)
    // 39       - no dither, very expensive 
    //
    // NOTES
    // -----------------------------------------------------------------------
    // 12 = slightly faster then FXAA 3.9 and higher edge quality (default)
    // 13 = about same speed as FXAA 3.9 and better than 12
    // 23 = closest to FXAA 3.9 visually and performance wise
    //  _ = the lowest digit is directly related to performance
    // _  = the highest digit is directly related to style
    // 
    #define FXAA_QUALITY__PRESET 12
#endif


/*============================================================================

                           FXAA QUALITY - PRESETS

============================================================================*/

/*============================================================================
                     FXAA QUALITY - MEDIUM DITHER PRESETS
============================================================================*/
#if (FXAA_QUALITY__PRESET == 10)
    #define FXAA_QUALITY__PS 3
    #define FXAA_QUALITY__P0 1.5
    #define FXAA_QUALITY__P1 3.0
    #define FXAA_QUALITY__P2 12.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 11)
    #define FXAA_QUALITY__PS 4
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 3.0
    #define FXAA_QUALITY__P3 12.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 12)
    #define FXAA_QUALITY__PS 5
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 4.0
    #define FXAA_QUALITY__P4 12.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 13)
    #define FXAA_QUALITY__PS 6
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 4.0
    #define FXAA_QUALITY__P5 12.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 14)
    #define FXAA_QUALITY__PS 7
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 4.0
    #define FXAA_QUALITY__P6 12.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 15)
    #define FXAA_QUALITY__PS 8
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 4.0
    #define FXAA_QUALITY__P7 12.0
#endif

/*============================================================================
                     FXAA QUALITY - LOW DITHER PRESETS
============================================================================*/
#if (FXAA_QUALITY__PRESET == 20)
    #define FXAA_QUALITY__PS 3
    #define FXAA_QUALITY__P0 1.5
    #define FXAA_QUALITY__P1 2.0
    #define FXAA_QUALITY__P2 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 21)
    #define FXAA_QUALITY__PS 4
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 22)
    #define FXAA_QUALITY__PS 5
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 23)
    #define FXAA_QUALITY__PS 6
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 24)
    #define FXAA_QUALITY__PS 7
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 3.0
    #define FXAA_QUALITY__P6 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 25)
    #define FXAA_QUALITY__PS 8
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 4.0
    #define FXAA_QUALITY__P7 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 26)
    #define FXAA_QUALITY__PS 9
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 2.0
    #define FXAA_QUALITY__P7 4.0
    #define FXAA_QUALITY__P8 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 27)
    #define FXAA_QUALITY__PS 10
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 2.0
    #define FXAA_QUALITY__P7 2.0
    #define FXAA_QUALITY__P8 4.0
    #define FXAA_QUALITY__P9 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 28)
    #define FXAA_QUALITY__PS 11
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 2.0
    #define FXAA_QUALITY__P7 2.0
    #define FXAA_QUALITY__P8 2.0
    #define FXAA_QUALITY__P9 4.0
    #define FXAA_QUALITY__P10 8.0
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_QUALITY__PRESET == 29)
    #define FXAA_QUALITY__PS 12
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.5
    #define FXAA_QUALITY__P2 2.0
    #define FXAA_QUALITY__P3 2.0
    #define FXAA_QUALITY__P4 2.0
    #define FXAA_QUALITY__P5 2.0
    #define FXAA_QUALITY__P6 2.0
    #define FXAA_QUALITY__P7 2.0
    #define FXAA_QUALITY__P8 2.0
    #define FXAA_QUALITY__P9 2.0
    #define FXAA_QUALITY__P10 4.0
    #define FXAA_QUALITY__P11 8.0
#endif

/*============================================================================
                     FXAA QUALITY - EXTREME QUALITY
============================================================================*/
#if (FXAA_QUALITY__PRESET == 39)
    #define FXAA_QUALITY__PS 12
    #define FXAA_QUALITY__P0 1.0
    #define FXAA_QUALITY__P1 1.0
    #define FXAA_QUALITY__P2 1.0
    #define FXAA_QUALITY__P3 1.0
    #define FXAA_QUALITY__P4 1.0
    #define FXAA_QUALITY__P5 1.5
    #define FXAA_QUALITY__P6 2.0
    #define FXAA_QUALITY__P7 2.0
    #define FXAA_QUALITY__P8 2.0
    #define FXAA_QUALITY__P9 2.0
    #define FXAA_QUALITY__P10 4.0
    #define FXAA_QUALITY__P11 8.0
#endif



/*============================================================================

                                API PORTING

============================================================================*/
#if (FXAA_GLSL_120 == 1) || (FXAA_GLSL_130 == 1)
    #define FxaaBool bool
    #define FxaaDiscard discard
    #define FxaaFloat float
    #define FxaaFloat2 vec2
    #define FxaaFloat3 vec3
    #define FxaaFloat4 vec4
    #define FxaaHalf float
    #define FxaaHalf2 vec2
    #define FxaaHalf3 vec3
    #define FxaaHalf4 vec4
    #define FxaaInt2 ivec2
    #define FxaaSat(x) clamp(x, 0.0, 1.0)
    #define FxaaTex sampler2D
#else
    #define FxaaBool bool
    #define FxaaDiscard clip(-1)
    #define FxaaFloat float
    #define FxaaFloat2 float2
    #define FxaaFloat3 float3
    #define FxaaFloat4 float4
    #define FxaaHalf half
    #define FxaaHalf2 half2
    #define FxaaHalf3 half3
    #define FxaaHalf4 half4
    #define FxaaSat(x) saturate(x)
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_GLSL_120 == 1)
    // Requires,
    //  #version 120
    // And at least,
    //  #extension GL_EXT_gpu_shader4 : enable
    //  (or set FXAA_FAST_PIXEL_OFFSET 1 to work like DX9)
    #define FxaaTexTop(t, p) texture2DLod(t, p, 0.0)
    #if (FXAA_FAST_PIXEL_OFFSET == 1)
        #define FxaaTexOff(t, p, o, r) texture2DLodOffset(t, p, 0.0, o)
    #else
        #define FxaaTexOff(t, p, o, r) texture2DLod(t, p + (o * r), 0.0)
    #endif
    #if (FXAA_GATHER4_ALPHA == 1)
        // use #extension GL_ARB_gpu_shader5 : enable
        #define FxaaTexAlpha4(t, p) textureGather(t, p, 3)
        #define FxaaTexOffAlpha4(t, p, o) textureGatherOffset(t, p, o, 3)
        #define FxaaTexGreen4(t, p) textureGather(t, p, 1)
        #define FxaaTexOffGreen4(t, p, o) textureGatherOffset(t, p, o, 1)
    #endif
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_GLSL_130 == 1)
    // Requires "#version 130" or better
    #define FxaaTexTop(t, p) textureLod(t, p, 0.0)
    #define FxaaTexOff(t, p, o, r) textureLodOffset(t, p, 0.0, o)
    #if (FXAA_GATHER4_ALPHA == 1)
        // use #extension GL_ARB_gpu_shader5 : enable
        #define FxaaTexAlpha4(t, p) textureGather(t, p, 3)
        #define FxaaTexOffAlpha4(t, p, o) textureGatherOffset(t, p, o, 3)
        #define FxaaTexGreen4(t, p) textureGather(t, p, 1)
        #define FxaaTexOffGreen4(t, p, o) textureGatherOffset(t, p, o, 1)
    #endif
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_HLSL_3 == 1) || (FXAA_360 == 1) || (FXAA_PS3 == 1)
    #define FxaaInt2 float2
    #define FxaaTex sampler2D
    #define FxaaTexTop(t, p) tex2Dlod(t, float4(p, 0.0, 0.0))
    #define FxaaTexOff(t, p, o, r) tex2Dlod(t, float4(p + (o * r), 0, 0))
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_HLSL_4 == 1)
    #define FxaaInt2 int2
    struct FxaaTex { SamplerState smpl; Texture2D tex; float4 UVMinMax; };
    #define FxaaTexTop(t, p) t.tex.SampleLevel(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), 0.0)
    #define FxaaTexOff(t, p, o, r) t.tex.SampleLevel(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), 0.0, o)
#endif
/*--------------------------------------------------------------------------*/
#if (FXAA_HLSL_5 == 1)
    #define FxaaInt2 int2
    struct FxaaTex { SamplerState smpl; Texture2D tex; float4 UVMinMax; };
    #define FxaaTexTop(t, p) t.tex.SampleLevel(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), 0.0)
    #define FxaaTexOff(t, p, o, r) t.tex.SampleLevel(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), 0.0, o)
    #define FxaaTexAlpha4(t, p) t.tex.GatherAlpha(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw))
    #define FxaaTexOffAlpha4(t, p, o) t.tex.GatherAlpha(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), o)
    #define FxaaTexGreen4(t, p) t.tex.GatherGreen(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw))
    #define FxaaTexOffGreen4(t, p, o) t.tex.GatherGreen(t.smpl, clamp(p, t.UVMinMax.xy, t.UVMinMax.zw), o)
#endif


/*============================================================================
                   GREEN AS LUMA OPTION SUPPORT FUNCTION
============================================================================*/
#if (FXAA_GREEN_AS_LUMA == 0)

	#if 1
		// allows the tonemapper to not output Luma in alpha channel to use a R10G10BA2
		FxaaFloat FxaaLuma(FxaaFloat4 rgba)
		{
			return dot(rgba.rgb, FxaaFloat3(0.299f, 0.587f, 0.114f));
		}

	#else
		FxaaFloat FxaaLuma(FxaaFloat4 rgba) { return rgba.w; }

	#endif

#else
    FxaaFloat FxaaLuma(FxaaFloat4 rgba) { return rgba.y; }
#endif    




/*============================================================================

                             FXAA3 QUALITY - PC

============================================================================*/
#if (FXAA_PC == 1)
/*--------------------------------------------------------------------------*/
FxaaFloat4 FxaaPixelShader(
    //
    // Use noperspective interpolation here (turn off perspective interpolation).
    // {xy} = center of pixel
    FxaaFloat2 pos,
    //
    // Used only for FXAA Console, and not used on the 360 version.
    // Use noperspective interpolation here (turn off perspective interpolation).
    // {xy__} = upper left of pixel
    // {__zw} = lower right of pixel
    FxaaFloat4 fxaaConsolePosPos,
    //
    // Input color texture.
    // {rgb_} = color in linear or perceptual color space
    // if (FXAA_GREEN_AS_LUMA == 0)
    //     {___a} = luma in perceptual color space (not linear)
    FxaaTex tex,
    //
    // Only used on the optimized 360 version of FXAA Console.
    // For everything but 360, just use the same input here as for "tex".
    // For 360, same texture, just alias with a 2nd sampler.
    // This sampler needs to have an exponent bias of -1.
    FxaaTex fxaaConsole360TexExpBiasNegOne,
    //
    // Only used on the optimized 360 version of FXAA Console.
    // For everything but 360, just use the same input here as for "tex".
    // For 360, same texture, just alias with a 3nd sampler.
    // This sampler needs to have an exponent bias of -2.
    FxaaTex fxaaConsole360TexExpBiasNegTwo,
    //
    // Only used on FXAA Quality.
    // This must be from a constant/uniform.
    // {x_} = 1.0/screenWidthInPixels
    // {_y} = 1.0/screenHeightInPixels
    FxaaFloat2 fxaaQualityRcpFrame,
    //
    // Only used on FXAA Console.
    // This must be from a constant/uniform.
    // This effects sub-pixel AA quality and inversely sharpness.
    //   Where N ranges between,
    //     N = 0.50 (default)
    //     N = 0.33 (sharper)
    // {x___} = -N/screenWidthInPixels  
    // {_y__} = -N/screenHeightInPixels
    // {__z_} =  N/screenWidthInPixels  
    // {___w} =  N/screenHeightInPixels 
    FxaaFloat4 fxaaConsoleRcpFrameOpt,
    //
    // Only used on FXAA Console.
    // Not used on 360, but used on PS3 and PC.
    // This must be from a constant/uniform.
    // {x___} = -2.0/screenWidthInPixels  
    // {_y__} = -2.0/screenHeightInPixels
    // {__z_} =  2.0/screenWidthInPixels  
    // {___w} =  2.0/screenHeightInPixels 
    FxaaFloat4 fxaaConsoleRcpFrameOpt2,
    //
    // Only used on FXAA Console.
    // Only used on 360 in place of fxaaConsoleRcpFrameOpt2.
    // This must be from a constant/uniform.
    // {x___} =  8.0/screenWidthInPixels  
    // {_y__} =  8.0/screenHeightInPixels
    // {__z_} = -4.0/screenWidthInPixels  
    // {___w} = -4.0/screenHeightInPixels 
    FxaaFloat4 fxaaConsole360RcpFrameOpt2,
    //
    // Only used on FXAA Quality.
    // This used to be the FXAA_QUALITY__SUBPIX define.
    // It is here now to allow easier tuning.
    // Choose the amount of sub-pixel aliasing removal.
    // This can effect sharpness.
    //   1.00 - upper limit (softer)
    //   0.75 - default amount of filtering
    //   0.50 - lower limit (sharper, less sub-pixel aliasing removal)
    //   0.25 - almost off
    //   0.00 - completely off
    FxaaFloat fxaaQualitySubpix,
    //
    // Only used on FXAA Quality.
    // This used to be the FXAA_QUALITY__EDGE_THRESHOLD define.
    // It is here now to allow easier tuning.
    // The minimum amount of local contrast required to apply algorithm.
    //   0.333 - too little (faster)
    //   0.250 - low quality
    //   0.166 - default
    //   0.125 - high quality 
    //   0.063 - overkill (slower)
    FxaaFloat fxaaQualityEdgeThreshold,
    //
    // Only used on FXAA Quality.
    // This used to be the FXAA_QUALITY__EDGE_THRESHOLD_MIN define.
    // It is here now to allow easier tuning.
    // Trims the algorithm from processing darks.
    //   0.0833 - upper limit (default, the start of visible unfiltered edges)
    //   0.0625 - high quality (faster)
    //   0.0312 - visible limit (slower)
    // Special notes when using FXAA_GREEN_AS_LUMA,
    //   Likely want to set this to zero.
    //   As colors that are mostly not-green
    //   will appear very dark in the green channel!
    //   Tune by looking at mostly non-green content,
    //   then start at zero and increase until aliasing is a problem.
    FxaaFloat fxaaQualityEdgeThresholdMin,
    // 
    // Only used on FXAA Console.
    // This used to be the FXAA_CONSOLE__EDGE_SHARPNESS define.
    // It is here now to allow easier tuning.
    // This does not effect PS3, as this needs to be compiled in.
    //   Use FXAA_CONSOLE__PS3_EDGE_SHARPNESS for PS3.
    //   Due to the PS3 being ALU bound,
    //   there are only three safe values here: 2 and 4 and 8.
    //   These options use the shaders ability to a free *|/ by 2|4|8.
    // For all other platforms can be a non-power of two.
    //   8.0 is sharper (default!!!)
    //   4.0 is softer
    //   2.0 is really soft (good only for vector graphics inputs)
    FxaaFloat fxaaConsoleEdgeSharpness,
    //
    // Only used on FXAA Console.
    // This used to be the FXAA_CONSOLE__EDGE_THRESHOLD define.
    // It is here now to allow easier tuning.
    // This does not effect PS3, as this needs to be compiled in.
    //   Use FXAA_CONSOLE__PS3_EDGE_THRESHOLD for PS3.
    //   Due to the PS3 being ALU bound,
    //   there are only two safe values here: 1/4 and 1/8.
    //   These options use the shaders ability to a free *|/ by 2|4|8.
    // The console setting has a different mapping than the quality setting.
    // Other platforms can use other values.
    //   0.125 leaves less aliasing, but is softer (default!!!)
    //   0.25 leaves more aliasing, and is sharper
    FxaaFloat fxaaConsoleEdgeThreshold,
    //
    // Only used on FXAA Console.
    // This used to be the FXAA_CONSOLE__EDGE_THRESHOLD_MIN define.
    // It is here now to allow easier tuning.
    // Trims the algorithm from processing darks.
    // The console setting has a different mapping than the quality setting.
    // This only applies when FXAA_EARLY_EXIT is 1.
    // This does not apply to PS3, 
    // PS3 was simplified to avoid more shader instructions.
    //   0.06 - faster but more aliasing in darks
    //   0.05 - default
    //   0.04 - slower and less aliasing in darks
    // Special notes when using FXAA_GREEN_AS_LUMA,
    //   Likely want to set this to zero.
    //   As colors that are mostly not-green
    //   will appear very dark in the green channel!
    //   Tune by looking at mostly non-green content,
    //   then start at zero and increase until aliasing is a problem.
    FxaaFloat fxaaConsoleEdgeThresholdMin,
    //    
    // Extra constants for 360 FXAA Console only.
    // Use zeros or anything else for other platforms.
    // These must be in physical constant registers and NOT immedates.
    // Immedates will result in compiler un-optimizing.
    // {xyzw} = float4(1.0, -1.0, 0.25, -0.25)
    FxaaFloat4 fxaaConsole360ConstDir
) {
/*--------------------------------------------------------------------------*/
    FxaaFloat2 posM;
    posM.x = pos.x;
    posM.y = pos.y;
    #if (FXAA_GATHER4_ALPHA == 1)
        #if (FXAA_DISCARD == 0)
            FxaaFloat4 rgbyM = FxaaTexTop(tex, posM);
            #if (FXAA_GREEN_AS_LUMA == 0)
                #define lumaM FxaaLuma(rgbyM)
            #else
                #define lumaM rgbyM.y
            #endif
        #endif
        #if (FXAA_GREEN_AS_LUMA == 0)
            FxaaFloat4 luma4A = FxaaTexAlpha4(tex, posM);
            FxaaFloat4 luma4B = FxaaTexOffAlpha4(tex, posM, FxaaInt2(-1, -1));
        #else
            FxaaFloat4 luma4A = FxaaTexGreen4(tex, posM);
            FxaaFloat4 luma4B = FxaaTexOffGreen4(tex, posM, FxaaInt2(-1, -1));
        #endif
        #if (FXAA_DISCARD == 1)
            #define lumaM luma4A.w
        #endif
        #define lumaE luma4A.z
        #define lumaS luma4A.x
        #define lumaSE luma4A.y
        #define lumaNW luma4B.w
        #define lumaN luma4B.z
        #define lumaW luma4B.x
    #else
        FxaaFloat4 rgbyM = FxaaTexTop(tex, posM);
        #if (FXAA_GREEN_AS_LUMA == 0)
            #define lumaM FxaaLuma(rgbyM)
        #else
            #define lumaM rgbyM.y
        #endif
        FxaaFloat lumaS = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2( 0, 1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaE = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2( 1, 0), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaN = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2( 0,-1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaW = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2(-1, 0), fxaaQualityRcpFrame.xy));
    #endif
/*--------------------------------------------------------------------------*/
    FxaaFloat maxSM = max(lumaS, lumaM);
    FxaaFloat minSM = min(lumaS, lumaM);
    FxaaFloat maxESM = max(lumaE, maxSM);
    FxaaFloat minESM = min(lumaE, minSM);
    FxaaFloat maxWN = max(lumaN, lumaW);
    FxaaFloat minWN = min(lumaN, lumaW);
    FxaaFloat rangeMax = max(maxWN, maxESM);
    FxaaFloat rangeMin = min(minWN, minESM);
    FxaaFloat rangeMaxScaled = rangeMax * fxaaQualityEdgeThreshold;
    FxaaFloat range = rangeMax - rangeMin;
    FxaaFloat rangeMaxClamped = max(fxaaQualityEdgeThresholdMin, rangeMaxScaled);
    FxaaBool earlyExit = range < rangeMaxClamped;
/*--------------------------------------------------------------------------*/
    if(earlyExit)
        #if (FXAA_DISCARD == 1)
            FxaaDiscard;
        #else
            return rgbyM;
        #endif
/*--------------------------------------------------------------------------*/
    #if (FXAA_GATHER4_ALPHA == 0)
        FxaaFloat lumaNW = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2(-1,-1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaSE = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2( 1, 1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaNE = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2( 1,-1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaSW = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2(-1, 1), fxaaQualityRcpFrame.xy));
    #else
        FxaaFloat lumaNE = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2(1, -1), fxaaQualityRcpFrame.xy));
        FxaaFloat lumaSW = FxaaLuma(FxaaTexOff(tex, posM, FxaaInt2(-1, 1), fxaaQualityRcpFrame.xy));
    #endif
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaNS = lumaN + lumaS;
    FxaaFloat lumaWE = lumaW + lumaE;
    FxaaFloat subpixRcpRange = 1.0/range;
    FxaaFloat subpixNSWE = lumaNS + lumaWE;
    FxaaFloat edgeHorz1 = (-2.0 * lumaM) + lumaNS;
    FxaaFloat edgeVert1 = (-2.0 * lumaM) + lumaWE;
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaNESE = lumaNE + lumaSE;
    FxaaFloat lumaNWNE = lumaNW + lumaNE;
    FxaaFloat edgeHorz2 = (-2.0 * lumaE) + lumaNESE;
    FxaaFloat edgeVert2 = (-2.0 * lumaN) + lumaNWNE;
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaNWSW = lumaNW + lumaSW;
    FxaaFloat lumaSWSE = lumaSW + lumaSE;
    FxaaFloat edgeHorz4 = (abs(edgeHorz1) * 2.0) + abs(edgeHorz2);
    FxaaFloat edgeVert4 = (abs(edgeVert1) * 2.0) + abs(edgeVert2);
    FxaaFloat edgeHorz3 = (-2.0 * lumaW) + lumaNWSW;
    FxaaFloat edgeVert3 = (-2.0 * lumaS) + lumaSWSE;
    FxaaFloat edgeHorz = abs(edgeHorz3) + edgeHorz4;
    FxaaFloat edgeVert = abs(edgeVert3) + edgeVert4;
/*--------------------------------------------------------------------------*/
    FxaaFloat subpixNWSWNESE = lumaNWSW + lumaNESE;
    FxaaFloat lengthSign = fxaaQualityRcpFrame.x;
    FxaaBool horzSpan = edgeHorz >= edgeVert;
    FxaaFloat subpixA = subpixNSWE * 2.0 + subpixNWSWNESE;
/*--------------------------------------------------------------------------*/
    if(!horzSpan) lumaN = lumaW;
    if(!horzSpan) lumaS = lumaE;
    if(horzSpan) lengthSign = fxaaQualityRcpFrame.y;
    FxaaFloat subpixB = (subpixA * (1.0/12.0)) - lumaM;
/*--------------------------------------------------------------------------*/
    FxaaFloat gradientN = lumaN - lumaM;
    FxaaFloat gradientS = lumaS - lumaM;
    FxaaFloat lumaNN = lumaN + lumaM;
    FxaaFloat lumaSS = lumaS + lumaM;
    FxaaBool pairN = abs(gradientN) >= abs(gradientS);
    FxaaFloat gradient = max(abs(gradientN), abs(gradientS));
    if(pairN) lengthSign = -lengthSign;
    FxaaFloat subpixC = FxaaSat(abs(subpixB) * subpixRcpRange);
/*--------------------------------------------------------------------------*/
    FxaaFloat2 posB;
    posB.x = posM.x;
    posB.y = posM.y;
    FxaaFloat2 offNP;
    offNP.x = (!horzSpan) ? 0.0 : fxaaQualityRcpFrame.x;
    offNP.y = ( horzSpan) ? 0.0 : fxaaQualityRcpFrame.y;
    if(!horzSpan) posB.x += lengthSign * 0.5;
    if( horzSpan) posB.y += lengthSign * 0.5;
/*--------------------------------------------------------------------------*/
    FxaaFloat2 posN;
    posN.x = posB.x - offNP.x * FXAA_QUALITY__P0;
    posN.y = posB.y - offNP.y * FXAA_QUALITY__P0;
    FxaaFloat2 posP;
    posP.x = posB.x + offNP.x * FXAA_QUALITY__P0;
    posP.y = posB.y + offNP.y * FXAA_QUALITY__P0;
    FxaaFloat subpixD = ((-2.0)*subpixC) + 3.0;
    FxaaFloat lumaEndN = FxaaLuma(FxaaTexTop(tex, posN));
    FxaaFloat subpixE = subpixC * subpixC;
    FxaaFloat lumaEndP = FxaaLuma(FxaaTexTop(tex, posP));
/*--------------------------------------------------------------------------*/
    if(!pairN) lumaNN = lumaSS;
    FxaaFloat gradientScaled = gradient * 1.0/4.0;
    FxaaFloat lumaMM = lumaM - lumaNN * 0.5;
    FxaaFloat subpixF = subpixD * subpixE;
    FxaaBool lumaMLTZero = lumaMM < 0.0;
/*--------------------------------------------------------------------------*/
    lumaEndN -= lumaNN * 0.5;
    lumaEndP -= lumaNN * 0.5;
    FxaaBool doneN = abs(lumaEndN) >= gradientScaled;
    FxaaBool doneP = abs(lumaEndP) >= gradientScaled;
    if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P1;
    if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P1;
    FxaaBool doneNP = (!doneN) || (!doneP);
    if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P1;
    if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P1;
/*--------------------------------------------------------------------------*/
    if(doneNP) {
        if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
        if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
        if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
        if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
        doneN = abs(lumaEndN) >= gradientScaled;
        doneP = abs(lumaEndP) >= gradientScaled;
        if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P2;
        if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P2;
        doneNP = (!doneN) || (!doneP);
        if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P2;
        if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P2;
/*--------------------------------------------------------------------------*/
        #if (FXAA_QUALITY__PS > 3)
        if(doneNP) {
            if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
            if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
            if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
            if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
            doneN = abs(lumaEndN) >= gradientScaled;
            doneP = abs(lumaEndP) >= gradientScaled;
            if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P3;
            if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P3;
            doneNP = (!doneN) || (!doneP);
            if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P3;
            if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P3;
/*--------------------------------------------------------------------------*/
            #if (FXAA_QUALITY__PS > 4)
            if(doneNP) {
                if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                doneN = abs(lumaEndN) >= gradientScaled;
                doneP = abs(lumaEndP) >= gradientScaled;
                if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P4;
                if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P4;
                doneNP = (!doneN) || (!doneP);
                if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P4;
                if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P4;
/*--------------------------------------------------------------------------*/
                #if (FXAA_QUALITY__PS > 5)
                if(doneNP) {
                    if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                    if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                    if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                    if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                    doneN = abs(lumaEndN) >= gradientScaled;
                    doneP = abs(lumaEndP) >= gradientScaled;
                    if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P5;
                    if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P5;
                    doneNP = (!doneN) || (!doneP);
                    if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P5;
                    if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P5;
/*--------------------------------------------------------------------------*/
                    #if (FXAA_QUALITY__PS > 6)
                    if(doneNP) {
                        if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                        if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                        if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                        if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                        doneN = abs(lumaEndN) >= gradientScaled;
                        doneP = abs(lumaEndP) >= gradientScaled;
                        if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P6;
                        if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P6;
                        doneNP = (!doneN) || (!doneP);
                        if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P6;
                        if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P6;
/*--------------------------------------------------------------------------*/
                        #if (FXAA_QUALITY__PS > 7)
                        if(doneNP) {
                            if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                            if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                            if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                            if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                            doneN = abs(lumaEndN) >= gradientScaled;
                            doneP = abs(lumaEndP) >= gradientScaled;
                            if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P7;
                            if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P7;
                            doneNP = (!doneN) || (!doneP);
                            if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P7;
                            if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P7;
/*--------------------------------------------------------------------------*/
    #if (FXAA_QUALITY__PS > 8)
    if(doneNP) {
        if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
        if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
        if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
        if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
        doneN = abs(lumaEndN) >= gradientScaled;
        doneP = abs(lumaEndP) >= gradientScaled;
        if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P8;
        if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P8;
        doneNP = (!doneN) || (!doneP);
        if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P8;
        if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P8;
/*--------------------------------------------------------------------------*/
        #if (FXAA_QUALITY__PS > 9)
        if(doneNP) {
            if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
            if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
            if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
            if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
            doneN = abs(lumaEndN) >= gradientScaled;
            doneP = abs(lumaEndP) >= gradientScaled;
            if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P9;
            if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P9;
            doneNP = (!doneN) || (!doneP);
            if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P9;
            if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P9;
/*--------------------------------------------------------------------------*/
            #if (FXAA_QUALITY__PS > 10)
            if(doneNP) {
                if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                doneN = abs(lumaEndN) >= gradientScaled;
                doneP = abs(lumaEndP) >= gradientScaled;
                if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P10;
                if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P10;
                doneNP = (!doneN) || (!doneP);
                if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P10;
                if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P10;
/*--------------------------------------------------------------------------*/
                #if (FXAA_QUALITY__PS > 11)
                if(doneNP) {
                    if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                    if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                    if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                    if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                    doneN = abs(lumaEndN) >= gradientScaled;
                    doneP = abs(lumaEndP) >= gradientScaled;
                    if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P11;
                    if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P11;
                    doneNP = (!doneN) || (!doneP);
                    if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P11;
                    if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P11;
/*--------------------------------------------------------------------------*/
                    #if (FXAA_QUALITY__PS > 12)
                    if(doneNP) {
                        if(!doneN) lumaEndN = FxaaLuma(FxaaTexTop(tex, posN.xy));
                        if(!doneP) lumaEndP = FxaaLuma(FxaaTexTop(tex, posP.xy));
                        if(!doneN) lumaEndN = lumaEndN - lumaNN * 0.5;
                        if(!doneP) lumaEndP = lumaEndP - lumaNN * 0.5;
                        doneN = abs(lumaEndN) >= gradientScaled;
                        doneP = abs(lumaEndP) >= gradientScaled;
                        if(!doneN) posN.x -= offNP.x * FXAA_QUALITY__P12;
                        if(!doneN) posN.y -= offNP.y * FXAA_QUALITY__P12;
                        doneNP = (!doneN) || (!doneP);
                        if(!doneP) posP.x += offNP.x * FXAA_QUALITY__P12;
                        if(!doneP) posP.y += offNP.y * FXAA_QUALITY__P12;
/*--------------------------------------------------------------------------*/
                    }
                    #endif
/*--------------------------------------------------------------------------*/
                }
                #endif
/*--------------------------------------------------------------------------*/
            }
            #endif
/*--------------------------------------------------------------------------*/
        }
        #endif
/*--------------------------------------------------------------------------*/
    }
    #endif
/*--------------------------------------------------------------------------*/
                        }
                        #endif
/*--------------------------------------------------------------------------*/
                    }
                    #endif
/*--------------------------------------------------------------------------*/
                }
                #endif
/*--------------------------------------------------------------------------*/
            }
            #endif
/*--------------------------------------------------------------------------*/
        }
        #endif
/*--------------------------------------------------------------------------*/
    }
/*--------------------------------------------------------------------------*/
    FxaaFloat dstN = posM.x - posN.x;
    FxaaFloat dstP = posP.x - posM.x;
    if(!horzSpan) dstN = posM.y - posN.y;
    if(!horzSpan) dstP = posP.y - posM.y;
/*--------------------------------------------------------------------------*/
    FxaaBool goodSpanN = (lumaEndN < 0.0) != lumaMLTZero;
    FxaaFloat spanLength = (dstP + dstN);
    FxaaBool goodSpanP = (lumaEndP < 0.0) != lumaMLTZero;
    FxaaFloat spanLengthRcp = 1.0/spanLength;
/*--------------------------------------------------------------------------*/
    FxaaBool directionN = dstN < dstP;
    FxaaFloat dstMin = min(dstN, dstP);
    FxaaBool goodSpan = directionN ? goodSpanN : goodSpanP;
    FxaaFloat subpixG = subpixF * subpixF;
    FxaaFloat pixelOffset = (dstMin * (-spanLengthRcp)) + 0.5;
    FxaaFloat subpixH = subpixG * fxaaQualitySubpix;
/*--------------------------------------------------------------------------*/
    FxaaFloat pixelOffsetGood = goodSpan ? pixelOffset : 0.0;
    FxaaFloat pixelOffsetSubpix = max(pixelOffsetGood, subpixH);
    if(!horzSpan) posM.x += pixelOffsetSubpix * lengthSign;
    if( horzSpan) posM.y += pixelOffsetSubpix * lengthSign;
    #if ((FXAA_DISCARD == 1) || 1)
        return FxaaTexTop(tex, posM);
    #else
        return FxaaFloat4(FxaaTexTop(tex, posM).xyz, lumaM);
    #endif
}
/*==========================================================================*/
#endif




/*============================================================================

                         FXAA3 CONSOLE - PC VERSION
                         
------------------------------------------------------------------------------
Instead of using this on PC, I'd suggest just using FXAA Quality with
    #define FXAA_QUALITY__PRESET 10
Or 
    #define FXAA_QUALITY__PRESET 20
Either are higher qualilty and almost as fast as this on modern PC GPUs.
============================================================================*/
#if (FXAA_PC_CONSOLE == 1)
/*--------------------------------------------------------------------------*/
FxaaFloat4 FxaaPixelShader(
    // See FXAA Quality FxaaPixelShader() source for docs on Inputs!
    FxaaFloat2 pos,
    FxaaFloat4 fxaaConsolePosPos,
    FxaaTex tex,
    FxaaTex fxaaConsole360TexExpBiasNegOne,
    FxaaTex fxaaConsole360TexExpBiasNegTwo,
    FxaaFloat2 fxaaQualityRcpFrame,
    FxaaFloat4 fxaaConsoleRcpFrameOpt,
    FxaaFloat4 fxaaConsoleRcpFrameOpt2,
    FxaaFloat4 fxaaConsole360RcpFrameOpt2,
    FxaaFloat fxaaQualitySubpix,
    FxaaFloat fxaaQualityEdgeThreshold,
    FxaaFloat fxaaQualityEdgeThresholdMin,
    FxaaFloat fxaaConsoleEdgeSharpness,
    FxaaFloat fxaaConsoleEdgeThreshold,
    FxaaFloat fxaaConsoleEdgeThresholdMin,
    FxaaFloat4 fxaaConsole360ConstDir
) {
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaNw = FxaaLuma(FxaaTexTop(tex, fxaaConsolePosPos.xy));
    FxaaFloat lumaSw = FxaaLuma(FxaaTexTop(tex, fxaaConsolePosPos.xw));
    FxaaFloat lumaNe = FxaaLuma(FxaaTexTop(tex, fxaaConsolePosPos.zy));
    FxaaFloat lumaSe = FxaaLuma(FxaaTexTop(tex, fxaaConsolePosPos.zw));
/*--------------------------------------------------------------------------*/
    FxaaFloat4 rgbyM = FxaaTexTop(tex, pos.xy);
    #if (FXAA_GREEN_AS_LUMA == 0)
        FxaaFloat lumaM = FxaaLuma(rgbyM);
    #else
        FxaaFloat lumaM = rgbyM.y;
    #endif
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaMaxNwSw = max(lumaNw, lumaSw);
    lumaNe += 1.0/384.0;
    FxaaFloat lumaMinNwSw = min(lumaNw, lumaSw);
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaMaxNeSe = max(lumaNe, lumaSe);
    FxaaFloat lumaMinNeSe = min(lumaNe, lumaSe);
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaMax = max(lumaMaxNeSe, lumaMaxNwSw);
    FxaaFloat lumaMin = min(lumaMinNeSe, lumaMinNwSw);
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaMaxScaled = lumaMax * fxaaConsoleEdgeThreshold;
/*--------------------------------------------------------------------------*/
    FxaaFloat lumaMinM = min(lumaMin, lumaM);
    FxaaFloat lumaMaxScaledClamped = max(fxaaConsoleEdgeThresholdMin, lumaMaxScaled);
    FxaaFloat lumaMaxM = max(lumaMax, lumaM);
    FxaaFloat dirSwMinusNe = lumaSw - lumaNe;
    FxaaFloat lumaMaxSubMinM = lumaMaxM - lumaMinM;
    FxaaFloat dirSeMinusNw = lumaSe - lumaNw;
    if(lumaMaxSubMinM < lumaMaxScaledClamped) return rgbyM;
/*--------------------------------------------------------------------------*/
    FxaaFloat2 dir;
    dir.x = dirSwMinusNe + dirSeMinusNw;
    dir.y = dirSwMinusNe - dirSeMinusNw;
/*--------------------------------------------------------------------------*/
    FxaaFloat2 dir1 = normalize(dir.xy);
    FxaaFloat4 rgbyN1 = FxaaTexTop(tex, pos.xy - dir1 * fxaaConsoleRcpFrameOpt.zw);
    FxaaFloat4 rgbyP1 = FxaaTexTop(tex, pos.xy + dir1 * fxaaConsoleRcpFrameOpt.zw);
/*--------------------------------------------------------------------------*/
    FxaaFloat dirAbsMinTimesC = min(abs(dir1.x), abs(dir1.y)) * fxaaConsoleEdgeSharpness;
    FxaaFloat2 dir2 = clamp(dir1.xy / dirAbsMinTimesC, -2.0, 2.0);
/*--------------------------------------------------------------------------*/
    FxaaFloat4 rgbyN2 = FxaaTexTop(tex, pos.xy - dir2 * fxaaConsoleRcpFrameOpt2.zw);
    FxaaFloat4 rgbyP2 = FxaaTexTop(tex, pos.xy + dir2 * fxaaConsoleRcpFrameOpt2.zw);
/*--------------------------------------------------------------------------*/
    FxaaFloat4 rgbyA = rgbyN1 + rgbyP1;
    FxaaFloat4 rgbyB = ((rgbyN2 + rgbyP2) * 0.25) + (rgbyA * 0.25);
/*--------------------------------------------------------------------------*/
    #if (FXAA_GREEN_AS_LUMA == 0)
        FxaaBool twoTap = (FxaaLuma(rgbyB) < lumaMin) || (FxaaLuma(rgbyB) > lumaMax);
    #else
        FxaaBool twoTap = (rgbyB.y < lumaMin) || (rgbyB.y > lumaMax);
    #endif
    if(twoTap) rgbyB.xyz = rgbyA.xyz * 0.5;
    return rgbyB; }
/*==========================================================================*/
#endif



/*============================================================================

                      FXAA3 CONSOLE - 360 PIXEL SHADER 

------------------------------------------------------------------------------
This optimized version thanks to suggestions from Andy Luedke.
Should be fully tex bound in all cases.
As of the FXAA 3.11 release, I have still not tested this code,
however I fixed a bug which was in both FXAA 3.9 and FXAA 3.10.
And note this is replacing the old unoptimized version.
If it does not work, please let me know so I can fix it.
============================================================================*/
#if (FXAA_360 == 1)
/*--------------------------------------------------------------------------*/
[reduceTempRegUsage(4)]
float4 FxaaPixelShader(
    // See FXAA Quality FxaaPixelShader() source for docs on Inputs!
    FxaaFloat2 pos,
    FxaaFloat4 fxaaConsolePosPos,
    FxaaTex tex,
    FxaaTex fxaaConsole360TexExpBiasNegOne,
    FxaaTex fxaaConsole360TexExpBiasNegTwo,
    FxaaFloat2 fxaaQualityRcpFrame,
    FxaaFloat4 fxaaConsoleRcpFrameOpt,
    FxaaFloat4 fxaaConsoleRcpFrameOpt2,
    FxaaFloat4 fxaaConsole360RcpFrameOpt2,
    FxaaFloat fxaaQualitySubpix,
    FxaaFloat fxaaQualityEdgeThreshold,
    FxaaFloat fxaaQualityEdgeThresholdMin,
    FxaaFloat fxaaConsoleEdgeSharpness,
    FxaaFloat fxaaConsoleEdgeThreshold,
    FxaaFloat fxaaConsoleEdgeThresholdMin,
    FxaaFloat4 fxaaConsole360ConstDir
) {
/*--------------------------------------------------------------------------*/
    float4 lumaNwNeSwSe;
    #if (FXAA_GREEN_AS_LUMA == 0)
        asm { 
            tfetch2D lumaNwNeSwSe.w___, tex, pos.xy, OffsetX = -0.5, OffsetY = -0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe._w__, tex, pos.xy, OffsetX =  0.5, OffsetY = -0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe.__w_, tex, pos.xy, OffsetX = -0.5, OffsetY =  0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe.___w, tex, pos.xy, OffsetX =  0.5, OffsetY =  0.5, UseComputedLOD=false
        };
    #else
        asm { 
            tfetch2D lumaNwNeSwSe.y___, tex, pos.xy, OffsetX = -0.5, OffsetY = -0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe._y__, tex, pos.xy, OffsetX =  0.5, OffsetY = -0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe.__y_, tex, pos.xy, OffsetX = -0.5, OffsetY =  0.5, UseComputedLOD=false
            tfetch2D lumaNwNeSwSe.___y, tex, pos.xy, OffsetX =  0.5, OffsetY =  0.5, UseComputedLOD=false
        };
    #endif
/*--------------------------------------------------------------------------*/
    lumaNwNeSwSe.y += 1.0/384.0;
    float2 lumaMinTemp = min(lumaNwNeSwSe.xy, lumaNwNeSwSe.zw);
    float2 lumaMaxTemp = max(lumaNwNeSwSe.xy, lumaNwNeSwSe.zw);
    float lumaMin = min(lumaMinTemp.x, lumaMinTemp.y);
    float lumaMax = max(lumaMaxTemp.x, lumaMaxTemp.y);
/*--------------------------------------------------------------------------*/
    float4 rgbyM = tex2Dlod(tex, float4(pos.xy, 0.0, 0.0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        float lumaMinM = min(lumaMin, rgbyM.w);
        float lumaMaxM = max(lumaMax, rgbyM.w);
    #else
        float lumaMinM = min(lumaMin, rgbyM.y);
        float lumaMaxM = max(lumaMax, rgbyM.y);
    #endif        
    if((lumaMaxM - lumaMinM) < max(fxaaConsoleEdgeThresholdMin, lumaMax * fxaaConsoleEdgeThreshold)) return rgbyM;
/*--------------------------------------------------------------------------*/
    float2 dir;
    dir.x = dot(lumaNwNeSwSe, fxaaConsole360ConstDir.yyxx);
    dir.y = dot(lumaNwNeSwSe, fxaaConsole360ConstDir.xyxy);
    dir = normalize(dir);
/*--------------------------------------------------------------------------*/
    float4 dir1 = dir.xyxy * fxaaConsoleRcpFrameOpt.xyzw;
/*--------------------------------------------------------------------------*/
    float4 dir2;
    float dirAbsMinTimesC = min(abs(dir.x), abs(dir.y)) * fxaaConsoleEdgeSharpness;
    dir2 = saturate(fxaaConsole360ConstDir.zzww * dir.xyxy / dirAbsMinTimesC + 0.5);
    dir2 = dir2 * fxaaConsole360RcpFrameOpt2.xyxy + fxaaConsole360RcpFrameOpt2.zwzw;
/*--------------------------------------------------------------------------*/
    float4 rgbyN1 = tex2Dlod(fxaaConsole360TexExpBiasNegOne, float4(pos.xy + dir1.xy, 0.0, 0.0));
    float4 rgbyP1 = tex2Dlod(fxaaConsole360TexExpBiasNegOne, float4(pos.xy + dir1.zw, 0.0, 0.0));
    float4 rgbyN2 = tex2Dlod(fxaaConsole360TexExpBiasNegTwo, float4(pos.xy + dir2.xy, 0.0, 0.0));
    float4 rgbyP2 = tex2Dlod(fxaaConsole360TexExpBiasNegTwo, float4(pos.xy + dir2.zw, 0.0, 0.0));
/*--------------------------------------------------------------------------*/
    float4 rgbyA = rgbyN1 + rgbyP1;
    float4 rgbyB = rgbyN2 + rgbyP2 * 0.5 + rgbyA;
/*--------------------------------------------------------------------------*/
    float4 rgbyR = ((rgbyB.w - lumaMax) > 0.0) ? rgbyA : rgbyB;
    rgbyR = ((rgbyB.w - lumaMin) > 0.0) ? rgbyR : rgbyA;
    return rgbyR; }
/*==========================================================================*/
#endif



/*============================================================================

         FXAA3 CONSOLE - OPTIMIZED PS3 PIXEL SHADER (NO EARLY EXIT)

==============================================================================
The code below does not exactly match the assembly.
I have a feeling that 12 cycles is possible, but was not able to get there.
Might have to increase register count to get full performance.
Note this shader does not use perspective interpolation.

Use the following cgc options,

  --fenable-bx2 --fastmath --fastprecision --nofloatbindings

------------------------------------------------------------------------------
                             NVSHADERPERF OUTPUT
------------------------------------------------------------------------------
For reference and to aid in debug, output of NVShaderPerf should match this,

Shader to schedule:
  0: texpkb h0.w(TRUE), v5.zyxx, #0
  2: addh h2.z(TRUE), h0.w, constant(0.001953, 0.000000, 0.000000, 0.000000).x
  4: texpkb h0.w(TRUE), v5.xwxx, #0
  6: addh h0.z(TRUE), -h2, h0.w
  7: texpkb h1.w(TRUE), v5, #0
  9: addh h0.x(TRUE), h0.z, -h1.w
 10: addh h3.w(TRUE), h0.z, h1
 11: texpkb h2.w(TRUE), v5.zwzz, #0
 13: addh h0.z(TRUE), h3.w, -h2.w
 14: addh h0.x(TRUE), h2.w, h0
 15: nrmh h1.xz(TRUE), h0_n
 16: minh_m8 h0.x(TRUE), |h1|, |h1.z|
 17: maxh h4.w(TRUE), h0, h1
 18: divx h2.xy(TRUE), h1_n.xzzw, h0_n
 19: movr r1.zw(TRUE), v4.xxxy
 20: madr r2.xz(TRUE), -h1, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).zzww, r1.zzww
 22: minh h5.w(TRUE), h0, h1
 23: texpkb h0(TRUE), r2.xzxx, #0
 25: madr r0.zw(TRUE), h1.xzxz, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w), r1
 27: maxh h4.x(TRUE), h2.z, h2.w
 28: texpkb h1(TRUE), r0.zwzz, #0
 30: addh_d2 h1(TRUE), h0, h1
 31: madr r0.xy(TRUE), -h2, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).xyxx, r1.zwzz
 33: texpkb h0(TRUE), r0, #0
 35: minh h4.z(TRUE), h2, h2.w
 36: fenct TRUE
 37: madr r1.xy(TRUE), h2, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).xyxx, r1.zwzz
 39: texpkb h2(TRUE), r1, #0
 41: addh_d2 h0(TRUE), h0, h2
 42: maxh h2.w(TRUE), h4, h4.x
 43: minh h2.x(TRUE), h5.w, h4.z
 44: addh_d2 h0(TRUE), h0, h1
 45: slth h2.x(TRUE), h0.w, h2
 46: sgth h2.w(TRUE), h0, h2
 47: movh h0(TRUE), h0
 48: addx.c0 rc(TRUE), h2, h2.w
 49: movh h0(c0.NE.x), h1

IPU0 ------ Simplified schedule: --------
Pass |  Unit  |  uOp |  PC:  Op
-----+--------+------+-------------------------
   1 | SCT0/1 |  mov |   0:  TXLr h0.w, g[TEX1].zyxx, const.xxxx, TEX0;
     |    TEX |  txl |   0:  TXLr h0.w, g[TEX1].zyxx, const.xxxx, TEX0;
     |   SCB1 |  add |   2:  ADDh h2.z, h0.--w-, const.--x-;
     |        |      |
   2 | SCT0/1 |  mov |   4:  TXLr h0.w, g[TEX1].xwxx, const.xxxx, TEX0;
     |    TEX |  txl |   4:  TXLr h0.w, g[TEX1].xwxx, const.xxxx, TEX0;
     |   SCB1 |  add |   6:  ADDh h0.z,-h2, h0.--w-;
     |        |      |
   3 | SCT0/1 |  mov |   7:  TXLr h1.w, g[TEX1], const.xxxx, TEX0;
     |    TEX |  txl |   7:  TXLr h1.w, g[TEX1], const.xxxx, TEX0;
     |   SCB0 |  add |   9:  ADDh h0.x, h0.z---,-h1.w---;
     |   SCB1 |  add |  10:  ADDh h3.w, h0.---z, h1;
     |        |      |
   4 | SCT0/1 |  mov |  11:  TXLr h2.w, g[TEX1].zwzz, const.xxxx, TEX0;
     |    TEX |  txl |  11:  TXLr h2.w, g[TEX1].zwzz, const.xxxx, TEX0;
     |   SCB0 |  add |  14:  ADDh h0.x, h2.w---, h0;
     |   SCB1 |  add |  13:  ADDh h0.z, h3.--w-,-h2.--w-;
     |        |      |
   5 |   SCT1 |  mov |  15:  NRMh h1.xz, h0;
     |    SRB |  nrm |  15:  NRMh h1.xz, h0;
     |   SCB0 |  min |  16:  MINh*8 h0.x, |h1|, |h1.z---|;
     |   SCB1 |  max |  17:  MAXh h4.w, h0, h1;
     |        |      |
   6 |   SCT0 |  div |  18:  DIVx h2.xy, h1.xz--, h0;
     |   SCT1 |  mov |  19:  MOVr r1.zw, g[TEX0].--xy;
     |   SCB0 |  mad |  20:  MADr r2.xz,-h1, const.z-w-, r1.z-w-;
     |   SCB1 |  min |  22:  MINh h5.w, h0, h1;
     |        |      |
   7 | SCT0/1 |  mov |  23:  TXLr h0, r2.xzxx, const.xxxx, TEX0;
     |    TEX |  txl |  23:  TXLr h0, r2.xzxx, const.xxxx, TEX0;
     |   SCB0 |  max |  27:  MAXh h4.x, h2.z---, h2.w---;
     |   SCB1 |  mad |  25:  MADr r0.zw, h1.--xz, const, r1;
     |        |      |
   8 | SCT0/1 |  mov |  28:  TXLr h1, r0.zwzz, const.xxxx, TEX0;
     |    TEX |  txl |  28:  TXLr h1, r0.zwzz, const.xxxx, TEX0;
     | SCB0/1 |  add |  30:  ADDh/2 h1, h0, h1;
     |        |      |
   9 |   SCT0 |  mad |  31:  MADr r0.xy,-h2, const.xy--, r1.zw--;
     |   SCT1 |  mov |  33:  TXLr h0, r0, const.zzzz, TEX0;
     |    TEX |  txl |  33:  TXLr h0, r0, const.zzzz, TEX0;
     |   SCB1 |  min |  35:  MINh h4.z, h2, h2.--w-;
     |        |      |
  10 |   SCT0 |  mad |  37:  MADr r1.xy, h2, const.xy--, r1.zw--;
     |   SCT1 |  mov |  39:  TXLr h2, r1, const.zzzz, TEX0;
     |    TEX |  txl |  39:  TXLr h2, r1, const.zzzz, TEX0;
     | SCB0/1 |  add |  41:  ADDh/2 h0, h0, h2;
     |        |      |
  11 |   SCT0 |  min |  43:  MINh h2.x, h5.w---, h4.z---;
     |   SCT1 |  max |  42:  MAXh h2.w, h4, h4.---x;
     | SCB0/1 |  add |  44:  ADDh/2 h0, h0, h1;
     |        |      |
  12 |   SCT0 |  set |  45:  SLTh h2.x, h0.w---, h2;
     |   SCT1 |  set |  46:  SGTh h2.w, h0, h2;
     | SCB0/1 |  mul |  47:  MOVh h0, h0;
     |        |      |
  13 |   SCT0 |  mad |  48:  ADDxc0_s rc, h2, h2.w---;
     | SCB0/1 |  mul |  49:  MOVh h0(NE0.xxxx), h1;
 
Pass   SCT  TEX  SCB
  1:   0% 100%  25%
  2:   0% 100%  25%
  3:   0% 100%  50%
  4:   0% 100%  50%
  5:   0%   0%  50%
  6: 100%   0%  75%
  7:   0% 100%  75%
  8:   0% 100% 100%
  9:   0% 100%  25%
 10:   0% 100% 100%
 11:  50%   0% 100%
 12:  50%   0% 100%
 13:  25%   0% 100%

MEAN:  17%  61%  67%

Pass   SCT0  SCT1   TEX  SCB0  SCB1
  1:    0%    0%  100%    0%  100%
  2:    0%    0%  100%    0%  100%
  3:    0%    0%  100%  100%  100%
  4:    0%    0%  100%  100%  100%
  5:    0%    0%    0%  100%  100%
  6:  100%  100%    0%  100%  100%
  7:    0%    0%  100%  100%  100%
  8:    0%    0%  100%  100%  100%
  9:    0%    0%  100%    0%  100%
 10:    0%    0%  100%  100%  100%
 11:  100%  100%    0%  100%  100%
 12:  100%  100%    0%  100%  100%
 13:  100%    0%    0%  100%  100%

MEAN:   30%   23%   61%   76%  100%
Fragment Performance Setup: Driver RSX Compiler, GPU RSX, Flags 0x5
Results 13 cycles, 3 r regs, 923,076,923 pixels/s
============================================================================*/
#if (FXAA_PS3 == 1) && (FXAA_EARLY_EXIT == 0)
/*--------------------------------------------------------------------------*/
#pragma regcount 7
#pragma disablepc all
#pragma option O3
#pragma option OutColorPrec=fp16
#pragma texformat default RGBA8
/*==========================================================================*/
half4 FxaaPixelShader(
    // See FXAA Quality FxaaPixelShader() source for docs on Inputs!
    FxaaFloat2 pos,
    FxaaFloat4 fxaaConsolePosPos,
    FxaaTex tex,
    FxaaTex fxaaConsole360TexExpBiasNegOne,
    FxaaTex fxaaConsole360TexExpBiasNegTwo,
    FxaaFloat2 fxaaQualityRcpFrame,
    FxaaFloat4 fxaaConsoleRcpFrameOpt,
    FxaaFloat4 fxaaConsoleRcpFrameOpt2,
    FxaaFloat4 fxaaConsole360RcpFrameOpt2,
    FxaaFloat fxaaQualitySubpix,
    FxaaFloat fxaaQualityEdgeThreshold,
    FxaaFloat fxaaQualityEdgeThresholdMin,
    FxaaFloat fxaaConsoleEdgeSharpness,
    FxaaFloat fxaaConsoleEdgeThreshold,
    FxaaFloat fxaaConsoleEdgeThresholdMin,
    FxaaFloat4 fxaaConsole360ConstDir
) {
/*--------------------------------------------------------------------------*/
// (1)
    half4 dir;
    half4 lumaNe = h4tex2Dlod(tex, half4(fxaaConsolePosPos.zy, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        lumaNe.w += half(1.0/512.0);
        dir.x = -lumaNe.w;
        dir.z = -lumaNe.w;
    #else
        lumaNe.y += half(1.0/512.0);
        dir.x = -lumaNe.y;
        dir.z = -lumaNe.y;
    #endif
/*--------------------------------------------------------------------------*/
// (2)
    half4 lumaSw = h4tex2Dlod(tex, half4(fxaaConsolePosPos.xw, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        dir.x += lumaSw.w;
        dir.z += lumaSw.w;
    #else
        dir.x += lumaSw.y;
        dir.z += lumaSw.y;
    #endif        
/*--------------------------------------------------------------------------*/
// (3)
    half4 lumaNw = h4tex2Dlod(tex, half4(fxaaConsolePosPos.xy, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        dir.x -= lumaNw.w;
        dir.z += lumaNw.w;
    #else
        dir.x -= lumaNw.y;
        dir.z += lumaNw.y;
    #endif
/*--------------------------------------------------------------------------*/
// (4)
    half4 lumaSe = h4tex2Dlod(tex, half4(fxaaConsolePosPos.zw, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        dir.x += lumaSe.w;
        dir.z -= lumaSe.w;
    #else
        dir.x += lumaSe.y;
        dir.z -= lumaSe.y;
    #endif
/*--------------------------------------------------------------------------*/
// (5)
    half4 dir1_pos;
    dir1_pos.xy = normalize(dir.xyz).xz;
    half dirAbsMinTimesC = min(abs(dir1_pos.x), abs(dir1_pos.y)) * half(FXAA_CONSOLE__PS3_EDGE_SHARPNESS);
/*--------------------------------------------------------------------------*/
// (6)
    half4 dir2_pos;
    dir2_pos.xy = clamp(dir1_pos.xy / dirAbsMinTimesC, half(-2.0), half(2.0));
    dir1_pos.zw = pos.xy;
    dir2_pos.zw = pos.xy;
    half4 temp1N;
    temp1N.xy = dir1_pos.zw - dir1_pos.xy * fxaaConsoleRcpFrameOpt.zw;
/*--------------------------------------------------------------------------*/
// (7)
    temp1N = h4tex2Dlod(tex, half4(temp1N.xy, 0.0, 0.0));
    half4 rgby1;
    rgby1.xy = dir1_pos.zw + dir1_pos.xy * fxaaConsoleRcpFrameOpt.zw;
/*--------------------------------------------------------------------------*/
// (8)
    rgby1 = h4tex2Dlod(tex, half4(rgby1.xy, 0.0, 0.0));
    rgby1 = (temp1N + rgby1) * 0.5;
/*--------------------------------------------------------------------------*/
// (9)
    half4 temp2N;
    temp2N.xy = dir2_pos.zw - dir2_pos.xy * fxaaConsoleRcpFrameOpt2.zw;
    temp2N = h4tex2Dlod(tex, half4(temp2N.xy, 0.0, 0.0));
/*--------------------------------------------------------------------------*/
// (10)
    half4 rgby2;
    rgby2.xy = dir2_pos.zw + dir2_pos.xy * fxaaConsoleRcpFrameOpt2.zw;
    rgby2 = h4tex2Dlod(tex, half4(rgby2.xy, 0.0, 0.0));
    rgby2 = (temp2N + rgby2) * 0.5;
/*--------------------------------------------------------------------------*/
// (11)
    // compilier moves these scalar ops up to other cycles
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaMin = min(min(lumaNw.w, lumaSw.w), min(lumaNe.w, lumaSe.w));
        half lumaMax = max(max(lumaNw.w, lumaSw.w), max(lumaNe.w, lumaSe.w));
    #else
        half lumaMin = min(min(lumaNw.y, lumaSw.y), min(lumaNe.y, lumaSe.y));
        half lumaMax = max(max(lumaNw.y, lumaSw.y), max(lumaNe.y, lumaSe.y));
    #endif        
    rgby2 = (rgby2 + rgby1) * 0.5;
/*--------------------------------------------------------------------------*/
// (12)
    #if (FXAA_GREEN_AS_LUMA == 0)
        bool twoTapLt = rgby2.w < lumaMin;
        bool twoTapGt = rgby2.w > lumaMax;
    #else
        bool twoTapLt = rgby2.y < lumaMin;
        bool twoTapGt = rgby2.y > lumaMax;
    #endif
/*--------------------------------------------------------------------------*/
// (13)
    if(twoTapLt || twoTapGt) rgby2 = rgby1;
/*--------------------------------------------------------------------------*/
    return rgby2; }
/*==========================================================================*/
#endif



/*============================================================================

       FXAA3 CONSOLE - OPTIMIZED PS3 PIXEL SHADER (WITH EARLY EXIT)

==============================================================================
The code mostly matches the assembly.
I have a feeling that 14 cycles is possible, but was not able to get there.
Might have to increase register count to get full performance.
Note this shader does not use perspective interpolation.

Use the following cgc options,

 --fenable-bx2 --fastmath --fastprecision --nofloatbindings

Use of FXAA_GREEN_AS_LUMA currently adds a cycle (16 clks).
Will look at fixing this for FXAA 3.12.
------------------------------------------------------------------------------
                             NVSHADERPERF OUTPUT
------------------------------------------------------------------------------
For reference and to aid in debug, output of NVShaderPerf should match this,

Shader to schedule:
  0: texpkb h0.w(TRUE), v5.zyxx, #0
  2: addh h2.y(TRUE), h0.w, constant(0.001953, 0.000000, 0.000000, 0.000000).x
  4: texpkb h1.w(TRUE), v5.xwxx, #0
  6: addh h0.x(TRUE), h1.w, -h2.y
  7: texpkb h2.w(TRUE), v5.zwzz, #0
  9: minh h4.w(TRUE), h2.y, h2
 10: maxh h5.x(TRUE), h2.y, h2.w
 11: texpkb h0.w(TRUE), v5, #0
 13: addh h3.w(TRUE), -h0, h0.x
 14: addh h0.x(TRUE), h0.w, h0
 15: addh h0.z(TRUE), -h2.w, h0.x
 16: addh h0.x(TRUE), h2.w, h3.w
 17: minh h5.y(TRUE), h0.w, h1.w
 18: nrmh h2.xz(TRUE), h0_n
 19: minh_m8 h2.w(TRUE), |h2.x|, |h2.z|
 20: divx h4.xy(TRUE), h2_n.xzzw, h2_n.w
 21: movr r1.zw(TRUE), v4.xxxy
 22: maxh h2.w(TRUE), h0, h1
 23: fenct TRUE
 24: madr r0.xy(TRUE), -h2.xzzw, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).zwzz, r1.zwzz
 26: texpkb h0(TRUE), r0, #0
 28: maxh h5.x(TRUE), h2.w, h5
 29: minh h5.w(TRUE), h5.y, h4
 30: madr r1.xy(TRUE), h2.xzzw, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).zwzz, r1.zwzz
 32: texpkb h2(TRUE), r1, #0
 34: addh_d2 h2(TRUE), h0, h2
 35: texpkb h1(TRUE), v4, #0
 37: maxh h5.y(TRUE), h5.x, h1.w
 38: minh h4.w(TRUE), h1, h5
 39: madr r0.xy(TRUE), -h4, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).xyxx, r1.zwzz
 41: texpkb h0(TRUE), r0, #0
 43: addh_m8 h5.z(TRUE), h5.y, -h4.w
 44: madr r2.xy(TRUE), h4, constant(cConst5.x, cConst5.y, cConst5.z, cConst5.w).xyxx, r1.zwzz
 46: texpkb h3(TRUE), r2, #0
 48: addh_d2 h0(TRUE), h0, h3
 49: addh_d2 h3(TRUE), h0, h2
 50: movh h0(TRUE), h3
 51: slth h3.x(TRUE), h3.w, h5.w
 52: sgth h3.w(TRUE), h3, h5.x
 53: addx.c0 rc(TRUE), h3.x, h3
 54: slth.c0 rc(TRUE), h5.z, h5
 55: movh h0(c0.NE.w), h2
 56: movh h0(c0.NE.x), h1

IPU0 ------ Simplified schedule: --------
Pass |  Unit  |  uOp |  PC:  Op
-----+--------+------+-------------------------
   1 | SCT0/1 |  mov |   0:  TXLr h0.w, g[TEX1].zyxx, const.xxxx, TEX0;
     |    TEX |  txl |   0:  TXLr h0.w, g[TEX1].zyxx, const.xxxx, TEX0;
     |   SCB0 |  add |   2:  ADDh h2.y, h0.-w--, const.-x--;
     |        |      |
   2 | SCT0/1 |  mov |   4:  TXLr h1.w, g[TEX1].xwxx, const.xxxx, TEX0;
     |    TEX |  txl |   4:  TXLr h1.w, g[TEX1].xwxx, const.xxxx, TEX0;
     |   SCB0 |  add |   6:  ADDh h0.x, h1.w---,-h2.y---;
     |        |      |
   3 | SCT0/1 |  mov |   7:  TXLr h2.w, g[TEX1].zwzz, const.xxxx, TEX0;
     |    TEX |  txl |   7:  TXLr h2.w, g[TEX1].zwzz, const.xxxx, TEX0;
     |   SCB0 |  max |  10:  MAXh h5.x, h2.y---, h2.w---;
     |   SCB1 |  min |   9:  MINh h4.w, h2.---y, h2;
     |        |      |
   4 | SCT0/1 |  mov |  11:  TXLr h0.w, g[TEX1], const.xxxx, TEX0;
     |    TEX |  txl |  11:  TXLr h0.w, g[TEX1], const.xxxx, TEX0;
     |   SCB0 |  add |  14:  ADDh h0.x, h0.w---, h0;
     |   SCB1 |  add |  13:  ADDh h3.w,-h0, h0.---x;
     |        |      |
   5 |   SCT0 |  mad |  16:  ADDh h0.x, h2.w---, h3.w---;
     |   SCT1 |  mad |  15:  ADDh h0.z,-h2.--w-, h0.--x-;
     |   SCB0 |  min |  17:  MINh h5.y, h0.-w--, h1.-w--;
     |        |      |
   6 |   SCT1 |  mov |  18:  NRMh h2.xz, h0;
     |    SRB |  nrm |  18:  NRMh h2.xz, h0;
     |   SCB1 |  min |  19:  MINh*8 h2.w, |h2.---x|, |h2.---z|;
     |        |      |
   7 |   SCT0 |  div |  20:  DIVx h4.xy, h2.xz--, h2.ww--;
     |   SCT1 |  mov |  21:  MOVr r1.zw, g[TEX0].--xy;
     |   SCB1 |  max |  22:  MAXh h2.w, h0, h1;
     |        |      |
   8 |   SCT0 |  mad |  24:  MADr r0.xy,-h2.xz--, const.zw--, r1.zw--;
     |   SCT1 |  mov |  26:  TXLr h0, r0, const.xxxx, TEX0;
     |    TEX |  txl |  26:  TXLr h0, r0, const.xxxx, TEX0;
     |   SCB0 |  max |  28:  MAXh h5.x, h2.w---, h5;
     |   SCB1 |  min |  29:  MINh h5.w, h5.---y, h4;
     |        |      |
   9 |   SCT0 |  mad |  30:  MADr r1.xy, h2.xz--, const.zw--, r1.zw--;
     |   SCT1 |  mov |  32:  TXLr h2, r1, const.xxxx, TEX0;
     |    TEX |  txl |  32:  TXLr h2, r1, const.xxxx, TEX0;
     | SCB0/1 |  add |  34:  ADDh/2 h2, h0, h2;
     |        |      |
  10 | SCT0/1 |  mov |  35:  TXLr h1, g[TEX0], const.xxxx, TEX0;
     |    TEX |  txl |  35:  TXLr h1, g[TEX0], const.xxxx, TEX0;
     |   SCB0 |  max |  37:  MAXh h5.y, h5.-x--, h1.-w--;
     |   SCB1 |  min |  38:  MINh h4.w, h1, h5;
     |        |      |
  11 |   SCT0 |  mad |  39:  MADr r0.xy,-h4, const.xy--, r1.zw--;
     |   SCT1 |  mov |  41:  TXLr h0, r0, const.zzzz, TEX0;
     |    TEX |  txl |  41:  TXLr h0, r0, const.zzzz, TEX0;
     |   SCB0 |  mad |  44:  MADr r2.xy, h4, const.xy--, r1.zw--;
     |   SCB1 |  add |  43:  ADDh*8 h5.z, h5.--y-,-h4.--w-;
     |        |      |
  12 | SCT0/1 |  mov |  46:  TXLr h3, r2, const.xxxx, TEX0;
     |    TEX |  txl |  46:  TXLr h3, r2, const.xxxx, TEX0;
     | SCB0/1 |  add |  48:  ADDh/2 h0, h0, h3;
     |        |      |
  13 | SCT0/1 |  mad |  49:  ADDh/2 h3, h0, h2;
     | SCB0/1 |  mul |  50:  MOVh h0, h3;
     |        |      |
  14 |   SCT0 |  set |  51:  SLTh h3.x, h3.w---, h5.w---;
     |   SCT1 |  set |  52:  SGTh h3.w, h3, h5.---x;
     |   SCB0 |  set |  54:  SLThc0 rc, h5.z---, h5;
     |   SCB1 |  add |  53:  ADDxc0_s rc, h3.---x, h3;
     |        |      |
  15 | SCT0/1 |  mul |  55:  MOVh h0(NE0.wwww), h2;
     | SCB0/1 |  mul |  56:  MOVh h0(NE0.xxxx), h1;
 
Pass   SCT  TEX  SCB
  1:   0% 100%  25%
  2:   0% 100%  25%
  3:   0% 100%  50%
  4:   0% 100%  50%
  5:  50%   0%  25%
  6:   0%   0%  25%
  7: 100%   0%  25%
  8:   0% 100%  50%
  9:   0% 100% 100%
 10:   0% 100%  50%
 11:   0% 100%  75%
 12:   0% 100% 100%
 13: 100%   0% 100%
 14:  50%   0%  50%
 15: 100%   0% 100%

MEAN:  26%  60%  56%

Pass   SCT0  SCT1   TEX  SCB0  SCB1
  1:    0%    0%  100%  100%    0%
  2:    0%    0%  100%  100%    0%
  3:    0%    0%  100%  100%  100%
  4:    0%    0%  100%  100%  100%
  5:  100%  100%    0%  100%    0%
  6:    0%    0%    0%    0%  100%
  7:  100%  100%    0%    0%  100%
  8:    0%    0%  100%  100%  100%
  9:    0%    0%  100%  100%  100%
 10:    0%    0%  100%  100%  100%
 11:    0%    0%  100%  100%  100%
 12:    0%    0%  100%  100%  100%
 13:  100%  100%    0%  100%  100%
 14:  100%  100%    0%  100%  100%
 15:  100%  100%    0%  100%  100%

MEAN:   33%   33%   60%   86%   80%
Fragment Performance Setup: Driver RSX Compiler, GPU RSX, Flags 0x5
Results 15 cycles, 3 r regs, 800,000,000 pixels/s
============================================================================*/
#if (FXAA_PS3 == 1) && (FXAA_EARLY_EXIT == 1)
/*--------------------------------------------------------------------------*/
#pragma regcount 7
#pragma disablepc all
#pragma option O2
#pragma option OutColorPrec=fp16
#pragma texformat default RGBA8
/*==========================================================================*/
half4 FxaaPixelShader(
    // See FXAA Quality FxaaPixelShader() source for docs on Inputs!
    FxaaFloat2 pos,
    FxaaFloat4 fxaaConsolePosPos,
    FxaaTex tex,
    FxaaTex fxaaConsole360TexExpBiasNegOne,
    FxaaTex fxaaConsole360TexExpBiasNegTwo,
    FxaaFloat2 fxaaQualityRcpFrame,
    FxaaFloat4 fxaaConsoleRcpFrameOpt,
    FxaaFloat4 fxaaConsoleRcpFrameOpt2,
    FxaaFloat4 fxaaConsole360RcpFrameOpt2,
    FxaaFloat fxaaQualitySubpix,
    FxaaFloat fxaaQualityEdgeThreshold,
    FxaaFloat fxaaQualityEdgeThresholdMin,
    FxaaFloat fxaaConsoleEdgeSharpness,
    FxaaFloat fxaaConsoleEdgeThreshold,
    FxaaFloat fxaaConsoleEdgeThresholdMin,
    FxaaFloat4 fxaaConsole360ConstDir
) {
/*--------------------------------------------------------------------------*/
// (1)
    half4 rgbyNe = h4tex2Dlod(tex, half4(fxaaConsolePosPos.zy, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaNe = rgbyNe.w + half(1.0/512.0);
    #else
        half lumaNe = rgbyNe.y + half(1.0/512.0);
    #endif
/*--------------------------------------------------------------------------*/
// (2)
    half4 lumaSw = h4tex2Dlod(tex, half4(fxaaConsolePosPos.xw, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaSwNegNe = lumaSw.w - lumaNe;
    #else
        half lumaSwNegNe = lumaSw.y - lumaNe;
    #endif
/*--------------------------------------------------------------------------*/
// (3)
    half4 lumaNw = h4tex2Dlod(tex, half4(fxaaConsolePosPos.xy, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaMaxNwSw = max(lumaNw.w, lumaSw.w);
        half lumaMinNwSw = min(lumaNw.w, lumaSw.w);
    #else
        half lumaMaxNwSw = max(lumaNw.y, lumaSw.y);
        half lumaMinNwSw = min(lumaNw.y, lumaSw.y);
    #endif
/*--------------------------------------------------------------------------*/
// (4)
    half4 lumaSe = h4tex2Dlod(tex, half4(fxaaConsolePosPos.zw, 0, 0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        half dirZ =  lumaNw.w + lumaSwNegNe;
        half dirX = -lumaNw.w + lumaSwNegNe;
    #else
        half dirZ =  lumaNw.y + lumaSwNegNe;
        half dirX = -lumaNw.y + lumaSwNegNe;
    #endif
/*--------------------------------------------------------------------------*/
// (5)
    half3 dir;
    dir.y = 0.0;
    #if (FXAA_GREEN_AS_LUMA == 0)
        dir.x =  lumaSe.w + dirX;
        dir.z = -lumaSe.w + dirZ;
        half lumaMinNeSe = min(lumaNe, lumaSe.w);
    #else
        dir.x =  lumaSe.y + dirX;
        dir.z = -lumaSe.y + dirZ;
        half lumaMinNeSe = min(lumaNe, lumaSe.y);
    #endif
/*--------------------------------------------------------------------------*/
// (6)
    half4 dir1_pos;
    dir1_pos.xy = normalize(dir).xz;
    half dirAbsMinTimes8 = min(abs(dir1_pos.x), abs(dir1_pos.y)) * half(FXAA_CONSOLE__PS3_EDGE_SHARPNESS);
/*--------------------------------------------------------------------------*/
// (7)
    half4 dir2_pos;
    dir2_pos.xy = clamp(dir1_pos.xy / dirAbsMinTimes8, half(-2.0), half(2.0));
    dir1_pos.zw = pos.xy;
    dir2_pos.zw = pos.xy;
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaMaxNeSe = max(lumaNe, lumaSe.w);
    #else
        half lumaMaxNeSe = max(lumaNe, lumaSe.y);
    #endif
/*--------------------------------------------------------------------------*/
// (8)
    half4 temp1N;
    temp1N.xy = dir1_pos.zw - dir1_pos.xy * fxaaConsoleRcpFrameOpt.zw;
    temp1N = h4tex2Dlod(tex, half4(temp1N.xy, 0.0, 0.0));
    half lumaMax = max(lumaMaxNwSw, lumaMaxNeSe);
    half lumaMin = min(lumaMinNwSw, lumaMinNeSe);
/*--------------------------------------------------------------------------*/
// (9)
    half4 rgby1;
    rgby1.xy = dir1_pos.zw + dir1_pos.xy * fxaaConsoleRcpFrameOpt.zw;
    rgby1 = h4tex2Dlod(tex, half4(rgby1.xy, 0.0, 0.0));
    rgby1 = (temp1N + rgby1) * 0.5;
/*--------------------------------------------------------------------------*/
// (10)
    half4 rgbyM = h4tex2Dlod(tex, half4(pos.xy, 0.0, 0.0));
    #if (FXAA_GREEN_AS_LUMA == 0)
        half lumaMaxM = max(lumaMax, rgbyM.w);
        half lumaMinM = min(lumaMin, rgbyM.w);
    #else
        half lumaMaxM = max(lumaMax, rgbyM.y);
        half lumaMinM = min(lumaMin, rgbyM.y);
    #endif
/*--------------------------------------------------------------------------*/
// (11)
    half4 temp2N;
    temp2N.xy = dir2_pos.zw - dir2_pos.xy * fxaaConsoleRcpFrameOpt2.zw;
    temp2N = h4tex2Dlod(tex, half4(temp2N.xy, 0.0, 0.0));
    half4 rgby2;
    rgby2.xy = dir2_pos.zw + dir2_pos.xy * fxaaConsoleRcpFrameOpt2.zw;
    half lumaRangeM = (lumaMaxM - lumaMinM) / FXAA_CONSOLE__PS3_EDGE_THRESHOLD;
/*--------------------------------------------------------------------------*/
// (12)
    rgby2 = h4tex2Dlod(tex, half4(rgby2.xy, 0.0, 0.0));
    rgby2 = (temp2N + rgby2) * 0.5;
/*--------------------------------------------------------------------------*/
// (13)
    rgby2 = (rgby2 + rgby1) * 0.5;
/*--------------------------------------------------------------------------*/
// (14)
    #if (FXAA_GREEN_AS_LUMA == 0)
        bool twoTapLt = rgby2.w < lumaMin;
        bool twoTapGt = rgby2.w > lumaMax;
    #else
        bool twoTapLt = rgby2.y < lumaMin;
        bool twoTapGt = rgby2.y > lumaMax;
    #endif
    bool earlyExit = lumaRangeM < lumaMax;
    bool twoTap = twoTapLt || twoTapGt;
/*--------------------------------------------------------------------------*/
// (15)
    if(twoTap) rgby2 = rgby1;
    if(earlyExit) rgby2 = rgbyM;
/*--------------------------------------------------------------------------*/
    return rgby2; }
/*==========================================================================*/
#endif
