// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef ALLOW_SSS_MATERIAL_OVERRIDE
#define ALLOW_SSS_MATERIAL_OVERRIDE 1
#endif

#ifndef SUBSTRATE_SSS_MATERIAL_OVERRIDE
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 1
#endif

// Whether to use only unlit or default lit shading model
#ifndef FORCE_SIMPLE_SHADING
	#define FORCE_SIMPLE_SHADING 0
#endif

#include "SceneTexturesCommon.ush"
#include "SceneTextureParameters.ush"
#include "DeferredShadingCommon.ush"

float SquareInline(float X)
{
	return X * X;
}

float3 EncodeNormalHelper(float3 SrcNormal, float QuantizationBias)
{
	return SrcNormal * .5f + .5f;
}

float3 DecodeNormalHelper(float3 SrcNormal)
{
	return SrcNormal * 2.0f - 1.0f;
}


uint EncodeQuantize6(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 63.0f + .5f + QuantizationBias),63u);
}

float DecodeQuantize6(uint Value)
{
	return float(Value) / 63.0f;
}

uint EncodeQuantize6Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 63.0f + .5f + QuantizationBias),63u);
}

float DecodeQuantize6Sqrt(uint Value)
{
	return SquareInline(float(Value) / 63.0f);
}

uint EncodeQuantize5(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 31.0f + .5f + QuantizationBias),31u);
}

float DecodeQuantize5(uint Value)
{
	return float(Value) / 31.0f;
}

uint EncodeQuantize5Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 31.0f + .5f + QuantizationBias),31u);
}

float DecodeQuantize5Sqrt(uint Value)
{
	return SquareInline(float(Value) / 31.0f);
}

uint EncodeQuantize4(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 15.0f + .5f + QuantizationBias),15u);
}

float DecodeQuantize4(uint Value)
{
	return float(Value) / 15.0f;
}

uint EncodeQuantize4Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 15.0f + .5f + QuantizationBias),15u);
}

float DecodeQuantize4Sqrt(uint Value)
{
	return SquareInline(float(Value) / 15.0f);
}


uint EncodeQuantize3(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 7.0f + .5f + QuantizationBias),7u);
}

float DecodeQuantize3(uint Value)
{
	return float(Value) / 7.0f;
}

uint EncodeQuantize3Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 7.0f + .5f + QuantizationBias),7u);
}

float DecodeQuantize3Sqrt(uint Value)
{
	return SquareInline(float(Value) / 7.0f);
}

uint EncodeQuantize2(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 3.0f + .5f + QuantizationBias),3u);
}

float DecodeQuantize2(uint Value)
{
	return float(Value) / 3.0f;
}

uint EncodeQuantize2Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 3.0f + .5f + QuantizationBias),3u);
}

float DecodeQuantize2Sqrt(uint Value)
{
	return SquareInline(float(Value) / 3.0f);
}

uint EncodeQuantize1(float Value, float QuantizationBias)
{
	return min(uint(saturate(Value) * 1.0f + .5f + QuantizationBias),1u);
}

float DecodeQuantize1(uint Value)
{
	return float(Value) / 1.0f;
}

uint EncodeQuantize1Sqrt(float Value, float QuantizationBias)
{
	return min(uint(sqrt(saturate(Value)) * 1.0f + .5f + QuantizationBias),1u);
}

float DecodeQuantize1Sqrt(uint Value)
{
	return SquareInline(float(Value) / 1.0f);
}


uint3 EncodeQuantize565(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize5(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize6(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize5(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize565(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize5(Value.x);
	Ret.y = DecodeQuantize6(Value.y);
	Ret.z = DecodeQuantize5(Value.z);
	return Ret;
}

uint3 EncodeQuantize565Sqrt(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize5Sqrt(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize6Sqrt(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize5Sqrt(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize565Sqrt(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize5Sqrt(Value.x);
	Ret.y = DecodeQuantize6Sqrt(Value.y);
	Ret.z = DecodeQuantize5Sqrt(Value.z);
	return Ret;
}


uint3 EncodeQuantize444(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize4(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize4(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize4(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize444(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize4(Value.x);
	Ret.y = DecodeQuantize4(Value.y);
	Ret.z = DecodeQuantize4(Value.z);
	return Ret;
}

uint3 EncodeQuantize444Sqrt(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize4Sqrt(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize4Sqrt(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize4Sqrt(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize444Sqrt(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize4Sqrt(Value.x);
	Ret.y = DecodeQuantize4Sqrt(Value.y);
	Ret.z = DecodeQuantize4Sqrt(Value.z);
	return Ret;
}


uint3 EncodeQuantize332(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize3(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize3(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize2(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize332(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize3(Value.x);
	Ret.y = DecodeQuantize3(Value.y);
	Ret.z = DecodeQuantize2(Value.z);
	return Ret;
}

uint3 EncodeQuantize332Sqrt(float3 Value, float QuantizationBias)
{
	uint3 Ret;
	Ret.x = EncodeQuantize3Sqrt(Value.x,QuantizationBias);
	Ret.y = EncodeQuantize3Sqrt(Value.y,QuantizationBias);
	Ret.z = EncodeQuantize2Sqrt(Value.z,QuantizationBias);
	return Ret;
}

float3 DecodeQuantize332Sqrt(uint3 Value)
{
	float3 Ret;
	Ret.x = DecodeQuantize3Sqrt(Value.x);
	Ret.y = DecodeQuantize3Sqrt(Value.y);
	Ret.z = DecodeQuantize2Sqrt(Value.z);
	return Ret;
}


void EnvBRDFApproxFullyRoughHelper(inout float3 DiffuseColor, inout float3 SpecularColor)
{
	// Factors derived from EnvBRDFApprox( SpecularColor, 1, 1 ) == SpecularColor * 0.4524 - 0.0024
	DiffuseColor += SpecularColor * 0.45;
	SpecularColor = 0;
	// We do not modify Roughness here as this is done differently at different places.
}

void EnvBRDFApproxFullyRoughHelper(inout float3 DiffuseColor, inout float SpecularColor)
{
	DiffuseColor += SpecularColor * 0.45;
	SpecularColor = 0;
}


#define USES_GBUFFER_HELPER	(FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !FORWARD_SHADING)

#define FORCE_FULLY_ROUGH_HELPER (MATERIAL_FULLY_ROUGH)


// TODO: Move the BasePassPixelShader extra changes into here.
void GBufferPreEncode(inout FGBufferData GBuffer, bool bChecker, float GeometricAARoughness, inout half3 OriginalBaseColor, inout half OriginalSpecular, inout half OriginalMetallic, float QuantizationBias)
{
#if MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS
	GBuffer.Roughness = max(GBuffer.Roughness, GeometricAARoughness);

#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		GBuffer.CustomData.y = max(GBuffer.CustomData.y, GeometricAARoughness);
	}	
#endif
#endif

#if USES_GBUFFER_HELPER && (MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_EYE)
	// SubsurfaceProfile applies the BaseColor in a later pass. Any lighting output in the base pass needs
	// to separate specular and diffuse lighting in a checkerboard pattern
	if (UseSubsurfaceProfile(GBuffer.ShadingModelID))
	{
		const bool bCheckerboardRequired = View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0;
		OriginalBaseColor = View.bSubsurfacePostprocessEnabled ? float3(1, 1, 1) : OriginalBaseColor;
		if (bCheckerboardRequired)
		{
			// because we adjust the BaseColor here, we need StoredBaseColor

			// we apply the base color later in SubsurfaceRecombinePS()
			OriginalBaseColor = bChecker;
			// in SubsurfaceRecombinePS() does not multiply with Specular so we do it here
			GBuffer.SpecularColor *= !bChecker;
			OriginalSpecular *= !bChecker;
		}

	}
#endif

	GBuffer.DiffuseColor = OriginalBaseColor - OriginalBaseColor * OriginalMetallic;

#if USE_DEVELOPMENT_SHADERS
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		GBuffer.DiffuseColor = GBuffer.DiffuseColor * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
		GBuffer.SpecularColor = GBuffer.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
	}
#endif

    #if !FORCE_FULLY_ROUGH_HELPER
	if (View.RenderingReflectionCaptureMask) // Force material rendered in reflection capture to have an expanded albedo to try to be energy conservative (when specular is removed).
	#endif
	{
		EnvBRDFApproxFullyRoughHelper(GBuffer.DiffuseColor, GBuffer.SpecularColor);
		// When rendering reflection captures, GBuffer.Roughness is already forced to 1 using RoughnessOverrideParameter in GetMaterialRoughness.
	}


// should move this
#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
		GBuffer.GenericAO = float(GBuffer.DiffuseIndirectSampleOcclusion) * (1.0f / 255.0f);
#elif ALLOW_STATIC_LIGHTING
		// No space for AO. Multiply IndirectIrradiance by AO instead of storing.
		GBuffer.GenericAO = EncodeIndirectIrradiance(GBuffer.IndirectIrradiance * GBuffer.GBufferAO) + QuantizationBias * (1.0 / 255.0);
#else
		GBuffer.GenericAO = GBuffer.GBufferAO;
#endif



}

// Temporary extra copy of this function. The autogenerated GBuffer functions will require changing the include heirarchy, so copying this function to miminize
// the number of files that have to be touched on the initial checkin. Will remove duplicates once the first checkin is tested and stable.
void AdjustBaseColorAndSpecularColorForSubsurfaceProfileLightingCopyHack(inout float3 BaseColor, inout float3 SpecularColor, inout float Specular, bool bChecker)
{
	// If this function is called in translucent pass, we should not modify the BaseColor, instead directly read the base color as it is. 
	// The translucent pass is called after the main post process subsurface scattering pass, it will not affect previous logic.
#if MATERIALBLENDING_TRANSLUCENT || !ALLOW_SSS_MATERIAL_OVERRIDE || !SUBSTRATE_SSS_MATERIAL_OVERRIDE
	return;
#else

#if SUBSURFACE_CHANNEL_MODE == 0
	// If SUBSURFACE_CHANNEL_MODE is 0, we can't support full-resolution lighting, so we 
	// ignore View.bCheckerboardSubsurfaceProfileRendering
	const bool bCheckerboardRequired = View.bSubsurfacePostprocessEnabled > 0 && ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE;
#else
	const bool bCheckerboardRequired = View.bSubsurfacePostprocessEnabled > 0 && View.bCheckerboardSubsurfaceProfileRendering > 0 && ALLOW_SSS_MATERIAL_CHECKERBOARD_OVERRIDE;
	BaseColor = View.bSubsurfacePostprocessEnabled ? float3(1, 1, 1) : BaseColor;
#endif
	if (bCheckerboardRequired)
	{
		// because we adjust the BaseColor here, we need StoredBaseColor

		// we apply the base color later in SubsurfaceRecombinePS()
		BaseColor = bChecker;
		// in SubsurfaceRecombinePS() does not multiply with Specular so we do it here
		SpecularColor *= !bChecker;
		Specular *= !bChecker;
	}
#endif
}

#ifndef INDIRECT_SAMPLE_COUNT
	#define INDIRECT_SAMPLE_COUNT 8
#endif

void GBufferPostDecode(inout FGBufferData Ret, bool bChecker, bool bGetNormalizedNormal)
{
#if FORCE_SIMPLE_SHADING || SUBSTRATE_FASTPATH
	Ret.ShadingModelID = Ret.ShadingModelID == SHADINGMODELID_DEFAULT_LIT ? SHADINGMODELID_DEFAULT_LIT : SHADINGMODELID_UNLIT;
#endif

	Ret.CustomData = HasCustomGBufferData(Ret.ShadingModelID) ? Ret.CustomData : half(0.0f);

	// FirstPerson uses a bit in SelectiveOutputMask that is aliased with ZERO_PRECSHADOW_MASK when !ALLOW_STATIC_LIGHTING, so we explicitly skip this logic here.
#if ALLOW_STATIC_LIGHTING
	Ret.PrecomputedShadowFactors = !(Ret.SelectiveOutputMask & 0x2) ? Ret.PrecomputedShadowFactors : ((Ret.SelectiveOutputMask & 0x4) ? half(0.0f) : half(1.0f));
#else
	Ret.PrecomputedShadowFactors = half(1.0f);
#endif
	Ret.Velocity = !(Ret.SelectiveOutputMask & 0x8) ? Ret.Velocity : half(0.0f);
	bool bHasAnisotropy = (Ret.SelectiveOutputMask & 0x1);

	Ret.StoredBaseColor = Ret.BaseColor;
	Ret.StoredMetallic = Ret.Metallic;
	Ret.StoredSpecular = Ret.Specular;

#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
	Ret.DiffuseIndirectSampleOcclusion = 255 * Ret.GenericAO;
	Ret.GBufferAO = saturate(1.0 - float(countbits(Ret.DiffuseIndirectSampleOcclusion)) * rcp(float(INDIRECT_SAMPLE_COUNT)));
	Ret.IndirectIrradiance = 1;
#elif ALLOW_STATIC_LIGHTING
	Ret.GBufferAO = 1;
	Ret.DiffuseIndirectSampleOcclusion = 0x0;
	Ret.IndirectIrradiance = half(DecodeIndirectIrradiance(Ret.GenericAO.x));
#else
	Ret.GBufferAO = Ret.GenericAO;
	Ret.DiffuseIndirectSampleOcclusion = 0x0;
	Ret.IndirectIrradiance = 1;
#endif

	if(bGetNormalizedNormal)
	{
		Ret.WorldNormal = normalize(Ret.WorldNormal);
	}

	FLATTEN
	if (IsEyeModel(Ret.ShadingModelID))
	{
		Ret.Metallic = 0.0;
	#if IRIS_NORMAL
		Ret.Specular = 0.25;
	#endif
	}
	
	// derived from BaseColor, Metalness, Specular
	{
		Ret.SpecularColor = ComputeF0(Ret.Specular, Ret.BaseColor, Ret.Metallic);
	
		if (UseSubsurfaceProfile(Ret.ShadingModelID))
		{
			AdjustBaseColorAndSpecularColorForSubsurfaceProfileLightingCopyHack(Ret.BaseColor, Ret.SpecularColor, Ret.Specular, bChecker);
		}
	
		Ret.DiffuseColor = Ret.BaseColor - Ret.BaseColor * Ret.Metallic;
	
		#if USE_DEVELOPMENT_SHADERS
		{
			// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
			Ret.DiffuseColor = Ret.DiffuseColor * View.DiffuseOverrideParameter.www + View.DiffuseOverrideParameter.xyz;
			Ret.SpecularColor = Ret.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
		}
		#endif //USE_DEVELOPMENT_SHADERS
	}
	
	if (bHasAnisotropy)
	{
		Ret.WorldTangent = half3(DecodeNormal(Ret.WorldTangent));
		Ret.Anisotropy = half(Ret.Anisotropy * 2.0f - 1.0f);
	
		if(bGetNormalizedNormal)
		{
			Ret.WorldTangent = normalize(Ret.WorldTangent);
		}
	}
	else
	{
		Ret.WorldTangent = 0;
		Ret.Anisotropy = 0;
	}

	// This requires cleanup. Shader code that uses GBuffer.SelectiveOutputMask expects the outputmask to be in
	// bits [4:7], but it gets packed as bits [0:3] in the flexible gbuffer since we might move it around.
	Ret.SelectiveOutputMask = Ret.SelectiveOutputMask << 4;
}
