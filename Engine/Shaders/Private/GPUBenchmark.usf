// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUBenchmark.usf: PostProcessing Benchmark.
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"

Texture2D InputTexture;
SamplerState InputTextureSampler;

// Simple Quad Vertex Shader, used for pixel tests
#if VS_METHOD == 0

void MainBenchmarkVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 UV : ATTRIBUTE1,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	) 
{
	DrawRectangle(InPosition, UV, OutPosition, OutUV);
}

// Vertex Throughput Test
#elif VS_METHOD == 1

void MainBenchmarkVS(
	in float4 Arg0 : ATTRIBUTE0,
	in float4 Arg1 : ATTRIBUTE1,
	in float4 Arg2 : ATTRIBUTE2,
	in float4 Arg3 : ATTRIBUTE3,
	in float4 Arg4 : ATTRIBUTE4,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition = Arg0 + Arg1 + Arg2 + Arg3 + Arg4;
	OutUV = 0.0f;
}

#elif VS_METHOD == 2

void MainBenchmarkVS(
	in uint VertexID : SV_VertexID,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition = float4(VertexID, 0, 0, 0);
	OutUV = 0.0f;
}

#else
#error Invalid VS_METHOD
#endif

// pixel shader entry point
void MainPS(float2 InUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = 0;

#if PS_METHOD == 0
	// ALU heavy
	{
		// some dependency to the input texture
		OutColor = Texture2DSample(InputTexture, InputTextureSampler, InUV) * 0.99f;

		UNROLL for(int i = 0; i < 16; ++i)
		{
			// todo: use float4 MAD to get raw GPU performance (should be same for scalar and non scalar)
			float4 Value = PseudoRandom(InUV + float2(i * 0.001f, 0));

			OutColor.r += Value.r;
		}
	}
#elif PS_METHOD == 1
	// TEX heavy
	{
		UNROLL for(int i = 0; i < 16; ++i)
		{
			float4 Value = Texture2DSample(InputTexture, InputTextureSampler, InUV + float2(i * 0.0001f, 0));

			OutColor.r += Value.r;
		}
	}
#elif PS_METHOD == 2
	// dependent TEX heavy
	{
		UNROLL for(int i = 0; i < 16; ++i)
		{
			float4 Value = Texture2DSample(InputTexture, InputTextureSampler, InUV + float2(i * 0.001f, OutColor.r * 0.001f));

			OutColor.r += Value.r;
		}
	}
#elif PS_METHOD == 3
	// some dependency to the input texture
	OutColor = Texture2DSample(InputTexture, InputTextureSampler, InUV) * 0.99f;
#elif PS_METHOD == 4
	// Bandwidth heavy
	{
		float2 PixelPos = frac(InUV * 512.0f / 16.0f) * 16.0f;

		UNROLL for(int y = 0; y < 4; ++y)
		{
			UNROLL for(int x = 0; x < 4; ++x)
			{
				// should be bandwidth trashing enough to profile memory bandwidth
				float4 Value = Texture2DSample(InputTexture, InputTextureSampler, (PixelPos + float2(x, y)) * 16 / 512.0f);

				OutColor.r += Value.r;
			}
		}
	}
#elif PS_METHOD == 5
	// Simple Pixel Shader used when testing vertex throughput.
	// Do Nothing (OutColor is black).
#else
#error Invalid PS_METHOD
#endif

	// todo: Framebuffer blending test, clear test, vertex performance, draw call performance, constant buffer upload performance
}

