// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GPUFastFourierTransform.usf: Core Fast Fourier Transform Code
=============================================================================*/

#pragma once

#include "Common.ush"  // for ReverseBits32()



// Requires the following define:
// SCAN_LINE_LENGTH

// Map get the RADIX and NUMTHREADSX as a function of SCAN_LINE_LENGTH

#if MIXED_RADIX == 1 && SCAN_LINE_LENGTH > 8
	#define RADIX 8
#else

#if SCAN_LINE_LENGTH == 2
	#define RADIX  2
#endif

#if SCAN_LINE_LENGTH == 4
	#define RADIX 4
#endif

#if SCAN_LINE_LENGTH == 8
	#define RADIX 8
#endif

#if SCAN_LINE_LENGTH == 16
	#define RADIX 4
#endif

#if SCAN_LINE_LENGTH == 32
	#define RADIX 2
#endif

#if SCAN_LINE_LENGTH == 64
	#define RADIX 8
#endif 

#if SCAN_LINE_LENGTH == 128
	#define RADIX 2
#endif

#if SCAN_LINE_LENGTH == 256
	#define RADIX 4
#endif

#if SCAN_LINE_LENGTH == 512
	#define RADIX 8
#endif

#if SCAN_LINE_LENGTH == 1024
	#define RADIX 4
#endif

#if SCAN_LINE_LENGTH == 2048
	#define RADIX 2
#endif

#if SCAN_LINE_LENGTH == 4096
	#define RADIX 8
#endif

#endif

// NB: This is only needed for the not group shared stuff below.
// Need to make that work and re-factor to fix this.
#ifndef RADIX
	#define RADIX 4
	#define SCAN_LINE_LENGTH  1024
#endif 

#define NUMTHREADSX ( SCAN_LINE_LENGTH / RADIX )

#define STRIDE ( SCAN_LINE_LENGTH / RADIX )


// Simple utility function
float ConvertToLuma(in float3 ColorValue)
{
	// Rec 709 function for luma.
	return dot(ColorValue, float3(0.2126, 0.7152, 0.0722));
}


// BrightPixelGain.x = Min Threshold: only pixels brighter than this get boosted
// BrightPixelGain.y = Max Threshold: the maximal value brightness
// BrightPixelGain.z = Multiplier applied to brightness above Min Threshold

float3 BrightPixelGain;

bool FilterPixel(in float3 Filter, inout float4 PixelValue)
{
	bool bIsChanged = false;
	float Luma = ConvertToLuma(PixelValue.xyz);
	
	if (Luma > Filter.x)
	{
		float TargetLuma = Filter.z * (Luma - Filter.x) + Filter.x;
		TargetLuma  = min(TargetLuma, Filter.y);

		PixelValue.rgb *= (TargetLuma / Luma);
		bIsChanged = true;
	}

	return bIsChanged;
}


// The Actual FFT code written to use local registers.


#include "GPUFastFourierTransform2DCore.ush"


uint2 Coord(in uint2 P, in bool bHorizontal)
{
	return (bHorizontal) ? P : uint2(P.y, P.x);
}

uint BitReverse(in uint Idx, in uint BitRange)
{
	uint ReversedValue = reversebits(Idx);
		
	// Shift into correct range.
	// 32 = sizeof(uint) 
	ReversedValue >>= (32-BitRange);

	return ReversedValue;
}

void ScrubNaNs(inout float4 InOutValue, bool bScrubNaNs)
{
	if (bScrubNaNs)
	{
		InOutValue = -min(-InOutValue, float4(0.f, 0.f, 0.f,0.f));
	}
}

// Used to insure the convolution doesn't produce energy (i.e. normalizing the kernel)
//  NB: This function relies on knowledge of the transform order and data layout to identify the 'dc' term in the fft. 
void GetKernelSum(in Texture2DType KernelTexture, in bool bIsHorizontal, uint NumScanlines, inout Complex Integral[2])
{

	
	if (!bIsHorizontal)  // Since this is the second tranform the first must have been horizontal, assume the data layout from a 2-for-1
	{
		Integral[0] = KernelTexture[uint2(0, 0)].xz;  // RB sums
		Integral[1] = KernelTexture[uint2(NumScanlines-2, 0)].xz;
		
	}	
	else
	{
		Integral[0] = KernelTexture[uint2(0, 0)].xz;
		Integral[1] = KernelTexture[uint2(0, NumScanlines-2)].xz;
	}
}


#ifdef INCLUDE_GROUP_SHARED_COMPLEX_FFT

// ---------------------------------------------------------------------------------------------------------------------------------------
//				FFT Compute Shader for two channels of complex data in an image
//              and inverse. 
//              And Helpers
// ---------------------------------------------------------------------------------------------------------------------------------------

uint2 SrcRectMin;
uint2 SrcRectMax;
uint2 DstExtent;
uint4 DstRect;


// CS Entry Point:
// Compute shader that performs (two) complex FFTs of the data in 'SrcTexture'
//
// uint2 TransformDef; determines the type (forward / inverse) and direction of the transform.
//       TransformType & 1 == {1,0} transforms the data in the {Horizontal , Vertical} direction.
//       TransformType & 2 == {1,0} performs a {Forward , Inverse} transform.
//
// The 'r & g' channels are transformed as one complex signal 'r + I * g'
// The 'b & a' channels are transformed as a second complex signal 'b + I * a'
//
// uint2 SrcRectMin,  SrcRectMax; 
// define the subsection of 'SrcTexture' to be transformed.  
// 
// 
// The FFT transforms a signal with a power-of-two length N.  
// A "Horizontal / Vertical" transform,  will transform horizontal/vertical scanlines independently.  
// The scanlines have length N = NUMTHREADSX * RADIX.  
// 
// The input data is composed of scanlines from a windowed region of SrcTexture (SrcRectMin/Max).
// Since SrcRectMax - SrcRectMin  is generally smaller than N, the scanlines are padded with zero. 
//
// NB:   uint2 TransformSize is the target buffer size.
//       It is assumed that TransformSize.x == NUMTHREADSX * RADIX  
//                          TransformSize.y == Number of thread groups

uint TransformType;
[numthreads(NUMTHREADSX, 1, 1)]
void GroupSharedComplexFFTCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);

	// Threads are defined in a 1d array.

	const uint ThreadIdx = GroupThreadID.x;

	// The scan line this thread works on

	const uint ScanIdx  = GroupID.z;


	// The length of the signal to be transformed
	
	const uint SignalLength = SCAN_LINE_LENGTH;

	// The main memory access pattern for this thread.

	uint Head = ThreadIdx;
	const uint Stride = STRIDE;   
	 
	// Thread-local memory.  Reserve two arrays since we plit .xy and .zw channels   
	
	Complex LocalBuffer[2][RADIX];
	
	
	// Load the local memory from the source texture
	// LocalBuffer[0][] holds .xy,  LocalBuffer[1][] holds.zw
	
	
	// Read from an image buffer
	CopyDataSrcWindowToLocal(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, SrcRectMin, SrcRectMax);
	//FFTMemoryBarrier();
	
	// Fourier Transform  the data
	// This uses the group shared memory and has appropriate syncs
	
	GroupSharedFFT(bIsForward, LocalBuffer, SignalLength, ThreadIdx);

	// Copy data to target buffer

	
	// Write image space data.
	//FFTMemoryBarrier();
	CopyDataLocalToDstWindow(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, DstExtent);
}

#endif  // #ifdef INCLUDE_GROUP_SHARED_COMPLEX_FFT




#ifdef INCLUDE_GROUP_SHARED_TWO_FOR_ONE_FFT

// ---------------------------------------------------------------------------------------------------------------------------------------
//				FFT Compute Shader for four channels of real data in an image, resulting in four 1/2 lenght channels of complex data
//              and inverse.   
//              And needed helper functions             
// ---------------------------------------------------------------------------------------------------------------------------------------


void ModifyInput(inout Complex LocalBuffer[2][RADIX])
{

	for (uint r = 0; r < RADIX; ++r)
	{
		float4 InputColor = float4(LocalBuffer[0][r].x, LocalBuffer[0][r].y, LocalBuffer[1][r].x, LocalBuffer[1][r].y);

		bool bIsChanged = FilterPixel(BrightPixelGain, InputColor);
		
		if (bIsChanged)
		{
			LocalBuffer[0][r] = InputColor.xy;
			LocalBuffer[1][r] = InputColor.zw;
		}
	}
}
 

// CS Entry Point:
// Compute shader that performs (four) real FFTs of the data in 'SrcTexture' using group shared
// memory.
//
// uint2 TransformType; determines the type (forward / inverse) and direction of the transform.
//       TransformType & 1 == {1,0} transforms the data in the {Horizontal , Vertical} direction.
//       TransformType & 2 == {1,0} performs a {Forward , Inverse} transform.
//
// For the Forward Transform: 
//		The 'r & g' channels are transformed as one complex signal 'r + I * g'
//		The 'b & a' channels are transformed as a second complex signal 'b + I * a'
//		and then the two-for-one unpacking is applied.   so the output layout will
//      be columns of float4 = (R, B), followed by columns of float4 = (G, A)
//      where R,G,B,A are the transforms of r,g,b,a and are complex (stored as float2).
//
// The Inverse Transform:
//      Consumes two-for-one unpacked data (see above), and inverts the two-for-one transform,
//      to recover the original signal.
//
// uint2 SrcRectMin,  SrcRectMax; 
// define the subsection of 'SrcTexture' to be transformed.  
// 
// 
// The FFT transforms a signal with a power-of-two length N.  
// A "Horizontal / Vertical" transform,  will transform horizontal/vertical scanlines independently.  
// The scanlines have length N = NUMTHREADSX * RADIX.  
// 
// The input data is composed of scanlines from a windowed region of SrcTexture (SrcRectMin/Max).
// Since SrcRectMax - SrcRectMin  is generally smaller than N, the scanlines are padded with zero. 
//
// NB:   uint2 TransformSize is the target buffer size.
//       It is assumed that TransformSize.x == NUMTHREADSX * RADIX  
//                          TransformSize.y == Number of thread groups
uint TransformType;
uint2 SrcRectMin;
uint2 SrcRectMax;
uint4 DstRect;

StructuredBuffer<float4> DstPostFilterParameters;

[numthreads(NUMTHREADSX, 1, 1)]
void GroupSharedTwoForOneFFTCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);
	const bool bModifyInput  = (TransformType & 0x4);

	// Threads are defined in a 1d array.

	const uint ThreadIdx = GroupThreadID.x;

	// The (horizontal / vertical) scan line this thread works on

	const uint ScanIdx  = GroupID.z;


	// The length of the signal to be transformed
	
	const uint SignalLength = SCAN_LINE_LENGTH;

	// The main memory access pattern for this thread.

	uint Head = ThreadIdx;
	const uint Stride = STRIDE;   
	 
	// Thread-local memory.  Reserve two arrays since we plit .xy and .zw channels   
	
	Complex LocalBuffer[2][RADIX];
	
	
	// Load the local memory from the source texture
	// LocalBuffer[0][] holds .xy,  LocalBuffer[1][] holds.zw
	
	if (bIsForward) 
	{
		uint4 SrcRect;
		SrcRect.xy = SrcRectMin.xy;
		SrcRect.zw = SrcRectMax.xy;
		// Read from an image buffer
		
		CopyDataSrcWindowToLocal(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, SrcRect);

		// NB: need to revisit
		// Note: this only works for colors. Any negatives or Nans will be set to zero
		
		ScrubNANs(LocalBuffer);

		if (bModifyInput)
		{
			// The input colors may be modified to boost the bright pixels.
			ModifyInput(LocalBuffer);	
		}		
	}
	else 
	{
		// Read a frequency space buffer with two-for-one data layout
		ReadTwoForOneFrequencyData(bIsHorizontal, LocalBuffer, ScanIdx, Head, Stride, SignalLength);
	}
	
	// Fourier Transform  the data
	// This uses the group shared memory and has appropriate syncs
	
	GroupSharedFFT(bIsForward, LocalBuffer, SignalLength, ThreadIdx);

	// Copy data to target buffer

	if (bIsForward) 
	{
		// Write a frequency space buffer with two-for-one data layout
		// FFTMemoryBarrier();
		WriteTwoForOneFrequencyData(bIsHorizontal, LocalBuffer, ScanIdx, Head, Stride, SignalLength);
	}
	else
	{
		// Write image space data.
		// FFTMemoryBarrier();

		// This is specialized for images, where floating point errors may have
		// resulted in (very very small) negative color values.
		ScrubNANs(LocalBuffer);
		
		float4 Scale = DstPostFilterParameters[0];

		UNROLL
		for (uint r = 0; r < RADIX; r++)
		{
			LocalBuffer[0][r] *= Scale.xy;
			LocalBuffer[1][r] *= Scale.xw;
		}

		CopyDataLocalToDstWindow(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, DstRect);
	}
}

#endif // #ifdef INCLUDE_GROUP_SHARED_TWO_FOR_ONE_FFT





#ifdef INCLUDE_GROUP_SHARED_CONVOLUTION_WITH_TEXTURE

// ---------------------------------------------------------------------------------------------------------------------------------------
//				FFT Compute Shader for convolution of image with a pre-transformed kernel. 
//
//              This shader is really the second of three steps to perform a 2D image convolution.
//					1)  Horizontal TwoForOneFFT(RealImage) -> ComplexData0
//					2)  Vertical ConvolutionWithTexture(ComplexData0) -> ComplexData1
//					3)  Horizontal Inverse TwoForOneFFT(ComplexData1)  -> RealImage 
// 
//				This assumes the incoming buffer has a data layout of four 1/2-lenght channels of complex data.
//              that resulted from 1D transform of 4 real channels (i.e. the image).
//				Also assumes that the pre-transformed kernel was computed in the same way 
//              (e.g. Vertical ComplexFFT ( Horizontal TwoForOneFFT(Kernel))) 
//
//      
//              And needed helper functions
// ---------------------------------------------------------------------------------------------------------------------------------------

// Input SRV: 
Texture2DType FilterTexture;


// Loading the Filter texture ( here the pre-transformed convolutin kernel)
void CopyFilterTextureToFilterBuffer(inout Complex Filter[2][RADIX], bool bIsHorizontal, in uint N, in uint Head, in uint Stride, in uint ScanIdx)
{
	if (bIsHorizontal) 
	{
		uint2 Pixel = uint2(Head, ScanIdx);
		UNROLL
		for (uint r = 0; r < RADIX; ++r, Pixel.x += Stride)
		{	
			float4 TextureValue = FilterTexture[Pixel];
			Filter[0][r] = TextureValue.xy;
			Filter[1][r] = TextureValue.zw;
		}
	
	}
	else 
	{
		uint2 Pixel = uint2(ScanIdx, Head);
		UNROLL
		for (uint r = 0; r < RADIX; ++r, Pixel.y += Stride)
		{
			float4 TextureValue = FilterTexture[Pixel];
			Filter[0][r] = TextureValue.xy;
			Filter[1][r] = TextureValue.zw;
		}
	}

}

void ComplexMultTexture( bool bUseAlpha, bool bIsGAGroup,  in Complex Filter[2][RADIX], inout Complex LocalBuffer[2][RADIX])
{
	UNROLL for (uint r = 0; r < RADIX; ++r)
	{
		LocalBuffer[0][r] = ComplexMult(LocalBuffer[0][r], Filter[0][r]);
	}   
	if (bUseAlpha || !bIsGAGroup)
	{
		UNROLL for (uint r = 0; r < RADIX; ++r)
		{
			LocalBuffer[1][r] = ComplexMult(LocalBuffer[1][r], Filter[1][r]);
		}
	}
}
void ComplexMultTexture(bool bIsHorizontal, bool bUseAlpha, bool bIsGAGroup, in uint N, in uint Head, in uint Stride, in uint ScanIdx, inout Complex LocalBuffer[2][RADIX])
{
	Complex Filter[2][RADIX];
	CopyFilterTextureToFilterBuffer(Filter, bIsHorizontal, N, Head, Stride, ScanIdx);
	
	ComplexMultTexture( bUseAlpha, bIsGAGroup,  Filter, LocalBuffer);
}


//float4 FilterTint;
void ApplyTint(in Complex Tint, inout Complex LocalBuffer[2][RADIX])
{
	{ for (uint r = 0; r < RADIX; ++r) LocalBuffer[0][r] *= Tint.x; }
	{ for (uint r = 0; r < RADIX; ++r) LocalBuffer[1][r] *= Tint.y; }
}


// CS Entry Point:
// Compute shader that does a convolution by applying complex FFTs on the data in 'SrcTexture'
// multiplies a texture (the transform of the physical space kernel), and then inverts the transform.
// 
//  Expected usage:  
//           input buffer of 4 float channels.
//           1) Horizontal_Forward Two-For-One Transform
//           2) Convolution ( Vertical_Forward complex transform, complex multiply Vertical_Inverse transform)
//           3) Horizontal_Inverse Two-For-One Transform.
//
// uint2 TransformType; determines the type (forward / inverse) and direction of the first transform in the 
//       convolution (cf. Vertical_Forward in step 2 above).
//       
//       TransformType & 1 == {1,0} transforms the data in the {Horizontal , Vertical} direction.
//       TransformType & 2 == {1,0} performs a {Forward , Inverse} transform.
//
// NB: The kernel texture must have been the result of the same forward steps 
// e.g. Horizontal_Forward two-for-one followed by Vertical_Forward complex in the above example.
//
// uint2 SrcRectMin,  SrcRectMax; 
// define the subsection of 'SrcTexture' to be transformed.  
// 
// 
// The FFT steps transform a signal with a power-of-two length N.  
// A "Horizontal / Vertical" transform,  will transform horizontal/vertical scanlines independently.  
// The scanlines have length N = NUMTHREADSX * RADIX.  
// 
// The input data is composed of scanlines from a windowed region of SrcTexture (SrcRectMin/Max).
// Since SrcRectMax - SrcRectMin  is generally smaller than N, the scanlines are padded with zero. 
//
// NB:   uint2 TransformSize is the target buffer size.
//       It is assumed that TransformSize.x == NUMTHREADSX * RADIX  
//                          TransformSize.y == Number of thread groups
uint TransformType;
uint2 DstExtent;
uint2 SrcRectMin;
uint2 SrcRectMax;

[numthreads(NUMTHREADSX, 1, 1)]
void GSConvolutionWithTextureCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	// These are constant across all thread groups
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);
	const bool bUseAlpha     = (TransformType & 0x8);

	// Threads are defined in a 1d array.

	const uint ThreadIdx = GroupThreadID.x;

	// The scan line this thread works on

	const uint ScanIdx  = GroupID.z;
	 
	const uint NumScanlines = (bIsHorizontal) ? DstExtent.y : DstExtent.x;

	//const uint NumFrequencies = TransformSize.x * TransformSize.y;

	// The two-for-one transform results in a data layout with complex coefficients
	// R G B A (representing the 1-d transform of r g b a)
	// In half of the buffer 
	//       R = SrcTexture.xy, B = SrcTexture.zw 
	// and in the other half
	//       G = SrcTexture.xy, A = SrcTexture.zw
	// With this layout 
	// R/B = columns [0, .., NumScanlines/2 -1] 
	// G/A = columns [NumScanlines/2, .., NumScanlines-1]

	// This data is loaded into the LocalBuffer[2][RADIX]
	// as   
	// LocalBuffer[0] = {R | G};  
	// LocalBuffer[1] = {B | A};
	
	
	// The thread groups in this shader act on the columns. 


	// This thread group acts on the columns of G/A.
	const bool bIsGAGroup = (2 * ScanIdx > NumScanlines - 2 );
	
	// Force the alpha 'tint' to do nothing
	float4 FilterTint = float4(1, 1, 1, 1);
	const float2 Tint = (bIsGAGroup) ? float2(FilterTint.y, 1.f) : FilterTint.xz;

	// The length of the signal to be transformed
	
	const uint SignalLength = SCAN_LINE_LENGTH;

	// The main memory access pattern for this thread.

	uint Head = ThreadIdx;
	const uint Stride = STRIDE;   
	 
	// Thread-local memory.  Reserve two arrays since we plit .xy and .zw channels   
	
	Complex LocalBuffer[2][RADIX];
	
	// Load the filter
	Complex Filter[2][RADIX];
	CopyFilterTextureToFilterBuffer(Filter, bIsHorizontal, SignalLength, Head, Stride, ScanIdx);
	
	
	// Load the local memory from the source texture
	// LocalBuffer[0][] holds .xy,  LocalBuffer[1][] holds.zw

	CopyDataSrcWindowToLocal(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, SrcRectMin, SrcRectMax);


	// Fourier Transform  the data
	// This uses the group shared memory and has appropriate syncs
	
	GroupSharedFFT(bIsForward, LocalBuffer, SignalLength, Head);
	
	 
	// ---- Convolution in frequency space is a multiply.
	// Here we multiply against the transform of a physical space kernel, but special case the 
	// thread groups that are working on  Green and Alpha


	ComplexMultTexture( bUseAlpha, bIsGAGroup,  Filter, LocalBuffer);

	// The input kernel might not have been normalized.  
	// This applies the correct normalization to local buffer,

	{
		float2 Norm[2];
		GetKernelSum(FilterTexture, bIsHorizontal, NumScanlines, Norm);
		// redSum = Norm[0].x;  greenSum = Norm[0].y; blueSum = Norm[1].x; alphaSum = Norm[1].y
		
		// Normalize R & G
		{
			//float Normal = NormMax;
			float Normal = (bIsGAGroup) ? Norm[0].y :  Norm[0].x;

			for (uint r = 0; r < RADIX; ++r)
			{
				// This is the R or G channel
				LocalBuffer[0][r] /= Normal; 
			}
		}
		
		// Normalize B & A
		{
			//float Normal = AorBNorm;
			float Normal = (bIsGAGroup) ? Norm[1].y :  Norm[1].x;

			for (uint r = 0; r < RADIX; ++r)
			{
				// This is the B or A channel
				LocalBuffer[1][r] /= Normal; 
			}
		}
	}


	// ---- Transform back ---- //

	GroupSharedFFT(!bIsForward, LocalBuffer, SignalLength, Head);

	// Apply additional tinting to the convolution result

	ApplyTint(Tint, LocalBuffer);

	// Copy Data back to main memory (dst)
	//uint2 Extent = SrcRectMax - SrcRectMin;

	CopyDataLocalToDstWindow(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, DstExtent);
}

#endif  // #ifdef INCLUDE_GROUP_SHARED_CONVOLUTION_WITH_TEXTURE


// ---------------------------------------------------------------------------------------------------------------------------------------
//	     Follow Compute Shaders are used by the multi-pass variant of the FFT.  
//       The multipass is only used when the image scanlines are too large for the group shared memory
//       implementation and will be much slower than the group-shared versions.
//      
//       ReorderFFTPassCS() :
//       The multipass works by first performing a "Decimation in Time" that reorders the input data.
//       This reording is equivalent to recursively segratating elements into Odd / Even groups untill the group size
//       fits in the group shared memory..
//
//       GroupSharedSubComplexFFTCS() : 
//       Then a group-shared pass oppertates on each subgroup independently.
// 
//       ComplexFFTPassCS():
//       Followed by the appropriate number of "Butterfly" passes to join the results.
//  
//       PackTwoForOneFFTPassCS():
//       Should the input or result of complex FFT need to be interpreted real data,
//       a pass can either split or merge the data. 
// ---------------------------------------------------------------------------------------------------------------------------------------

float4 WindowRead(in uint2 Texel, in uint4 Window, in Texture2DType SrcTexture)
{
	// not in window
	bool bNotIn = !(Texel.x < Window.z) || 
		           (Texel.x < Window.x) || 
				  !(Texel.y < Window.w) ||
				   (Texel.y < Window.y);

    if (bNotIn)
	{
		return float4(0.f, 0.f, 0.f, 0.f);
	}
	else
	{
		return SrcTexture[Texel];
	}
}

#ifdef INCLUDE_REORDER_FFT_PASS

// Input SRV: 
Texture2DType   SrcSRV;
RWTexture2DType DstUAV;


#define NUMTHREADS_PER_COL 32

// This pass re-orders the data of lenght 2^LogTwoLength 
// into 2^BitCount disjoint regions in preparation for 2^BitCount
// independent FFTs.
// The partition is equivalent to recursively splitting into even and odd
// subregions.   
// Bit Count (1) :  even , odd entries
// Bit Count (2) :  even-even, odd -even, odd-odd.
// etc.

// e.g. input f(0), f(1), f(2), f(3), f(4), f(5), f(6), f(7)
// Bit Count 1 ->  f(0), f(2), f(4), f(6) . f(1), f(3), f(5), f(7)
// Bit Count 2 ->  f(0), f(4) . f(2), f(6) . f(1), f(5), f(3), f(7)

uint4 SrcRect;
uint4 DstRect;
uint TransformType;
uint TransformLenght;  
uint BitCount; // log(2, Pow2SubLenghtCount);
uint LogTwoLength; // log(2, TransformLength);

// Reverse the last 'BitReverseCount' values and move them to the 
// High bits.  Assumes BitReverse count < 32
uint PartialBitReverse(in uint InValue, in uint BitReverseCount, in uint BitRange )
{
	uint Result = 0;
	uint Tmp = InValue;
	for (uint i = 0; i < BitReverseCount; ++i)
	{
		uint LowBit = Tmp & 0x1;
		Tmp = Tmp >> 1;
		Result = Result << 1;
		Result |= LowBit;
	}
	Result = Result << (BitRange - BitReverseCount);
	Result |= Tmp;

	return Result;
}

uint InversePartialBitReverse(in uint InValue, in uint BitReverseCount, in uint BitRange)
{
	
	
	uint ResultValue;
	// -- for testing
	if (false && BitReverseCount == 1) 
	{
		uint HalfTransformLength = 1u << (BitRange -1);
		if (InValue < HalfTransformLength )
		{
			ResultValue = 2 * InValue;
		}
		else
		{
			ResultValue = 2 * (InValue - HalfTransformLength) + 1;
		}
	}
 	
	// This could be made faster..
	ResultValue = BitReverse(InValue, BitRange);
	ResultValue = PartialBitReverse(ResultValue, BitReverseCount, BitRange);
	ResultValue = BitReverse(ResultValue, BitRange);
	
	return ResultValue;
}

[numthreads(NUMTHREADS_PER_COL, 1, 1)]
void ReorderFFTPassCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	const uint ThreadsPerGroup = NUMTHREADS_PER_COL;
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);
	const bool bScrubNaNs    = (TransformType & 0x4);

	// The number of bit that need to be reversed.
	const uint BitReverseCount = BitCount;

	// The lenght of the transform.
	const uint TransformLenght = 1u << LogTwoLength;
	const uint NumSubLengths = 1u << BitCount;
	
	// When inverting the transform the correct scale is 1 / TransformLenght
	// But the group shared sub transforms will scale by NumSubLengths / TransformLength.
	// So we need to account for the additional factor here.

	float Scale = (bIsForward) ? 1.f : 1.f / float(NumSubLengths);


	// NB: The names assume we doing a horizontal transform.  In which case thread groups opperate on columns
	// of data.

	// Number of elements in the transverse direction.

	const uint NumRows = (bIsHorizontal) ? SrcRect.w - SrcRect.y : SrcRect.z - SrcRect.x;


	// All the threads in this group operate on this column.

	const uint ColNum  = GroupID.z;

	// This thread is responsible for the element (RowIdx, ColIdx)
	
	const uint RowIdx = GroupThreadID.x; 
	
	const uint DstColA = 2 * ColNum;	
	const uint DstColB = DstColA + 1;

	// Bit reverse to find the Src Columns
	const uint SrcColA =  InversePartialBitReverse(DstColA, BitCount, LogTwoLength);
	const uint SrcColB =  InversePartialBitReverse(DstColB, BitCount, LogTwoLength);
	

	
	const uint2 SrcOffset = SrcRect.xy;  
	const uint2 DstOffset = DstRect.xy;

	// Loop over the rows that this thread owns.
	//const uint ElementsPerThread = NumRows / ThreadsPerGroup;

	float4 SrcValueA;
	float4 SrcValueB;

	uint4 Window = SrcRect;
	
	if (bIsHorizontal)
	{
		uint CurRowIdx = RowIdx;
		for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
		{
			uint2 SrcTexelA = SrcOffset + uint2(SrcColA, CurRowIdx);
			uint2 SrcTexelB = SrcOffset + uint2(SrcColB, CurRowIdx);

			uint2 DstTexelA = DstOffset + uint2(DstColA, CurRowIdx);
			uint2 DstTexelB = DstOffset + uint2(DstColB, CurRowIdx);
		
				
			SrcValueA = WindowRead(SrcTexelA, Window, SrcSRV);
			ScrubNaNs(SrcValueA, bScrubNaNs);
	
			DstUAV[DstTexelA] = Scale * SrcValueA;	
			
			
			SrcValueB = WindowRead(SrcTexelB, Window, SrcSRV);	
			ScrubNaNs(SrcValueB, bScrubNaNs);	
			
			DstUAV[DstTexelB] = Scale * SrcValueB;
		}
	}
	else
	{
		uint CurRowIdx = RowIdx;
		for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
		{
			uint2 SrcTexelA = SrcOffset + uint2(CurRowIdx, SrcColA);
			uint2 SrcTexelB = SrcOffset + uint2(CurRowIdx, SrcColB);

			uint2 DstTexelA = DstOffset + uint2(CurRowIdx, DstColA);
			uint2 DstTexelB = DstOffset + uint2(CurRowIdx, DstColB);
		
			SrcValueB = WindowRead(SrcTexelB, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueB, bScrubNaNs);
			SrcValueA = WindowRead(SrcTexelA, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueA, bScrubNaNs);

			// Write out
			DstUAV[DstTexelA] = Scale * SrcValueA;	
			DstUAV[DstTexelB] = Scale * SrcValueB;
		}
	}
}

#endif //INCLUDE_REORDER_FFT_PASS


#ifdef INCLUDE_GROUP_SHARED_SUB_COMPLEX_FFT

// The FFT transforms a signal with a power-of-two length N.  
// A "Horizontal / Vertical" transform,  will transform horizontal/vertical scanlines independently.  
// The scanlines have length N = NUMTHREADSX * RADIX.  
// 
// The input data is composed of scanlines from a windowed region of SrcTexture (SrcRectMin/Max).
// Since SrcRectMax - SrcRectMin  is generally smaller than N, the scanlines are padded with zero. 
//
// NB:   uint2 TransformSize is the target buffer size.
//       It is assumed that TransformSize.x == NUMTHREADSX * RADIX  
//                          TransformSize.y == Number of thread groups
// Input SRV: 
//Texture2DType   SrcSRV;
//RWTexture2DType DstUAV;

uint NumSubRegions;
uint TransformLength;
uint4 SrcWindow;
uint TransformType;
[numthreads(NUMTHREADSX, 1, 1)]
void GroupSharedSubComplexFFTCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);

	// Threads are defined in a 1d array.

	const uint ThreadIdx = GroupThreadID.x;

	// The scan line this thread works on

	const uint ScanIdx  = GroupID.z;


	// The length of the signal to be transformed
	
	const uint SignalLength = SCAN_LINE_LENGTH;

	const uint SubLength = TransformLength / NumSubRegions;

	// The main memory access pattern for this thread.

	uint Head = ThreadIdx;
	const uint Stride = STRIDE;   
	 
	// Thread-local memory.  Reserve two arrays since we plit .xy and .zw channels   
	
	Complex LocalBuffer[2][RADIX];
	
	
	// Load the local memory from the source texture
	// LocalBuffer[0][] holds .xy,  LocalBuffer[1][] holds.zw
	
	uint4 Window    =  SrcWindow;
	uint2 WindowOffset = (bIsHorizontal) ? uint2(SubLength, 0) : uint2(0, SubLength);

	// Do 'NumSubRegions' independent (spatially disjoint) transforms that together cover the 
	// entire domain.
	
	for (uint SubRegionId = 0; SubRegionId < NumSubRegions; SubRegionId++)
	{
		// Create the correct src window for this read
		
		FFTMemoryBarrier();

		// Offset the Read & Write window.

		// Read from an image buffer
		CopyDataSrcWindowToLocal(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, Window.xy, Window.zw);
		//FFTMemoryBarrier();
	
		// Fourier Transform  the data
		// This uses the group shared memory and has appropriate syncs
		// NB: bIsForward == false case applies 1/SignalLenght scaling 
		// for the inverse.

		GroupSharedFFT(bIsForward, LocalBuffer, SignalLength, ThreadIdx);

		// Copy data to target buffer

		CopyDataLocalToDstWindow(LocalBuffer, bIsHorizontal, ScanIdx, Head, Stride, Window);
		
		// Move the window so we can work on the next section.
		Window = Window + uint4(WindowOffset, WindowOffset);
	}
}

#endif  // #ifdef INCLUDE_GROUP_SHARED_SUB_COMPLEX_FFT



#ifdef  INCLUDE_COMPLEX_FFT_PASS
// No group shared mememory FFT, does multiple passes


// Input SRV: 
Texture2DType   SrcSRV;
RWTexture2DType DstUAV;


#define NUMTHREADS_PER_COL 32 

uint4 SrcRect;
uint4 DstRect;
uint TransformType;
uint BitCount;  // log(2, TransformLength) + 1;
uint PowTwoLength;  // Power of two.  2^p where p is the pass number

StructuredBuffer<float4> DstPostFilterParameters;

[numthreads(NUMTHREADS_PER_COL, 1, 1)]
void ComplexFFTPassCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	const uint ThreadsPerGroup = NUMTHREADS_PER_COL;
	const bool bIsHorizontal = (TransformType & 0x1);
	const bool bIsForward    = (TransformType & 0x2);
	const bool bScrubNaNs    = (TransformType & 0x4);
	
	// We zipping together 2 transforms of length Ns to make a new one
	// of length 2*Ns 

	const uint Ns = PowTwoLength;
	const uint TwoNs = 2 * Ns;
	int Stride = Ns;

	//const bool bScrubNaNs = (bIsForward && Ns == 1);

	// NB: The names assume we doing a horizontal transform.  In which case thread groups opperate on columns
	// of data.

	// Number of elements in the transverse direction.

	const uint NumRows = (bIsHorizontal) ? SrcRect.w - SrcRect.y : SrcRect.z - SrcRect.x;


	// All the threads in this group operate on this column.

	const uint ColNum  = GroupID.z;

	// This thread is responsible for the element (RowIdx, ColIdx)
	
	const uint RowIdx = GroupThreadID.x;

	// The Radix-2 points needed for this pass.
	// Ns = 2^pass.  E.g. zero pass, Ns = 1: pass 1, Ns = 2, pass 2 Ns = 4... 
	
	uint SrcColA = (ColNum / Ns) * TwoNs + ColNum % Ns;
	uint SrcColB = SrcColA + Ns;
	
	const uint DstColA = SrcColA;	
	const uint DstColB = SrcColB;
	
	float Angle = 6.283185307179586f * (  float(SrcColA % TwoNs)   / float(TwoNs) );
	if (!bIsForward) {
		Angle *= -1.f;
	}

	Complex Twiddle = Complex(1.f, 1.f);
	sincos(Angle, Twiddle.y, Twiddle.x);
	
	// Bit reverse on the first pass
	if (Ns == 1)
	{
		SrcColA = BitReverse(DstColA, BitCount - 1);
		SrcColB = BitReverse(DstColB, BitCount - 1);
	}

	
	const uint2 SrcOffset = SrcRect.xy;  
	const uint2 DstOffset = DstRect.xy;

	// Loop over the rows that this thread owns.
	//const uint ElementsPerThread = NumRows / ThreadsPerGroup;

	float4 SrcValueA;
	float4 SrcValueB;

	const uint TransformLength = 1u << (BitCount-1);
	
	float4 Scale = DstPostFilterParameters[0] * ((Ns == 1 && !bIsForward) ? 1.f / float(TransformLength) : 1.f);

	if (bIsHorizontal)
	{
		uint CurRowIdx = RowIdx;
		for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
		{
			uint2 SrcTexelA = SrcOffset + uint2(SrcColA, CurRowIdx);
			uint2 SrcTexelB = SrcOffset + uint2(SrcColB, CurRowIdx);

			uint2 DstTexelA = DstOffset + uint2(DstColA, CurRowIdx);
			uint2 DstTexelB = DstOffset + uint2(DstColB, CurRowIdx);
		
			SrcValueB = WindowRead(SrcTexelB, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueB, bScrubNaNs);

			{
				float2 RG = SrcValueB.xy;
				float2 BA = SrcValueB.zw;
				RG = ComplexMult(Twiddle, RG);
				BA = ComplexMult(Twiddle, BA);
				SrcValueB = float4(RG.x, RG.y, BA.x, BA.y);
				//SrcValueB = float4(ComplexMult(Twiddle, SrcValueB.xy), ComplexMult(Twiddle, SrcValueB.zw));
			}
				
			SrcValueA = WindowRead(SrcTexelA, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueA, bScrubNaNs);

			float4 ResultValueA = SrcValueA + SrcValueB;
			float4 ResultValueB = SrcValueA - SrcValueB;

			ResultValueA *= Scale;
			ResultValueB *= Scale;
			

			// Write out
			DstUAV[DstTexelA] = ResultValueA;	
			DstUAV[DstTexelB] = ResultValueB;
		}
	}
	else
	{
		uint CurRowIdx = RowIdx;
		for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
		{
			uint2 SrcTexelA = SrcOffset + uint2(CurRowIdx, SrcColA);
			uint2 SrcTexelB = SrcOffset + uint2(CurRowIdx, SrcColB);

			uint2 DstTexelA = DstOffset + uint2(CurRowIdx, DstColA);
			uint2 DstTexelB = DstOffset + uint2(CurRowIdx, DstColB);
		
			SrcValueB = WindowRead(SrcTexelB, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueB, bScrubNaNs);
			SrcValueB = float4(ComplexMult(Twiddle, SrcValueB.xy), ComplexMult(Twiddle, SrcValueB.zw));
		
			SrcValueA = WindowRead(SrcTexelA, SrcRect, SrcSRV);
			ScrubNaNs(SrcValueA, bScrubNaNs);

			float4 ResultValueA = SrcValueA + SrcValueB;
			float4 ResultValueB = SrcValueA - SrcValueB;

			ResultValueA *= Scale;
			ResultValueB *= Scale;
			
			// Write out
			DstUAV[DstTexelA] = ResultValueA;	
			DstUAV[DstTexelB] = ResultValueB;
		}
	}
}
#endif  //ifdef   INCLUDE_COMPLEX_FFT_PASS

#ifdef INCLUDE_PACK_TWOFORONE_FFT_PASS

#define NUMTHREADS_PER_COL 512 


// Input SRV: 
Texture2DType   SrcSRV;
RWTexture2DType DstUAV;

uint4 DstRect;
uint TransformType;

[numthreads(NUMTHREADS_PER_COL, 1, 1)]
void PackTwoForOneFFTPassCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	// currently assume that the src and result have the same offset.
	uint4 SrcRect = DstRect;

	const uint ThreadsPerGroup = NUMTHREADS_PER_COL;
	const bool bIsHorizontal   = (TransformType & 0x1);
	const bool bIsForward      = (TransformType & 0x2);

	// Number of elements in the transverse direction.

	const uint NumRows = (bIsHorizontal) ? DstRect.w - DstRect.y : DstRect.z - DstRect.x;
	
	// All the threads in this group operate on this "column""
	// for forward: K in [0, Transform Lenght /2 + 1)
	// for inverse: K in [0, Transform Lenght /2 + 1)
	const uint K  = GroupID.z;	
	const uint RowIdx = GroupThreadID.x;

	// This thread is responsible for the element (K, RowIdx)


	uint NumDstCol = (bIsHorizontal) ? DstRect.z - DstRect.x : DstRect.w - DstRect.y;
	uint N = (bIsForward) ? NumDstCol - 2 : NumDstCol;
	uint Non2 = N / 2;

	// Forward case: Two Real Signals were transformed as a single complex signal, split them apart.
	if (bIsForward) // dispatch-level
	{
		uint Non2 = N / 2;
		
		//if (K > Non2) return;

		if (K != 0 && K != Non2)  // group-level
		{
			const uint NmK = N - K;

			uint CurRowIdx = RowIdx;
			for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
			{
				// I need values at K and N - K
				uint2 TexelK   = Coord( uint2(K, CurRowIdx),   bIsHorizontal );
				float4 ZAtK    = SrcSRV[ SrcRect.xy + TexelK ];
				  
				uint2 TexelNmK = Coord( uint2(NmK, CurRowIdx), bIsHorizontal );
				float4 ZAtNmK  = SrcSRV[ SrcRect.xy + TexelNmK ]; 
				
				{
					// F_k = (1/2) ( Z_k + Conjugate(Z_{N-k}) )
					float4 FAtK   = ZAtK + float4(ZAtNmK.x, -ZAtNmK.y, ZAtNmK.z, -ZAtNmK.w);
					FAtK *= float4(0.5f, 0.5f, 0.5f, 0.5f);
			
					DstUAV[DstRect.xy +TexelK] = FAtK;
				}

				{
					// F_{N-k} = -(i/2) (Z_{N-k} - Conjuate(Z_k))
					float4 FAtNmK  = ZAtNmK - float4(ZAtK.x, -ZAtK.y, ZAtK.z, -ZAtK.w);
					FAtNmK *= -float4(0.5f, 0.5f, 0.5f, 0.5f);
			
					// mult by 'i'
					FAtNmK = float4(-FAtNmK.y, FAtNmK.x, -FAtNmK.w, FAtNmK.z);
					DstUAV[DstRect.xy + TexelNmK] = FAtNmK;
				}
			}
		}
		else // K == 0 || K == N/2
		{
			uint CurRowIdx = RowIdx;
			for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
			{
				uint2 TexelK  = Coord( uint2(K, CurRowIdx), bIsHorizontal );
				float4 ZAtK   = SrcSRV[ SrcRect.xy + TexelK ]; 
				DstUAV[DstRect.xy + TexelK] = float4(ZAtK.x, 0.f, ZAtK.z, 0.f);

				uint2 Texel2 = ( K == 0 ) ? Coord(uint2(N, CurRowIdx), bIsHorizontal) :  Coord(uint2( N + 1, CurRowIdx), bIsHorizontal);
				
				DstUAV[DstRect.xy + Texel2]  = float4(ZAtK.y, 0.f, ZAtK.w, 0.f);
			}
		}

	} 
	else // Inverse case: Merge the coefficients from two real transforms int a single complex signal.
	{
		const uint Non2 = N / 2;
		// for the inverse K is in [0, N/2 + 1)
		if (K != 0 && K != Non2)
		{
			if (K > Non2) return;

			const uint NmK = N - K ;

			uint CurRowIdx = RowIdx;
			for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
			{
				// I need values at K and N - K  
		
				uint2 TexelK   = Coord(uint2(K,   CurRowIdx), bIsHorizontal);  
				uint2 TexelNmK = Coord(uint2(NmK, CurRowIdx), bIsHorizontal);

				float4 ZAtNmK = SrcSRV[ SrcRect.xy + TexelNmK]; 
				float4 ZAtK   = SrcSRV[ SrcRect.xy + TexelK ];
			
				{
					// Complex( SharedImag[ NmK ], SharedReal[ NmK ] )
					float4 FAtK   = ZAtK + float4(ZAtNmK.y, ZAtNmK.x, ZAtNmK.w, ZAtNmK.z);
					DstUAV[ DstRect.xy + TexelK ] = FAtK;
				}
				{
					float4 FAtNmK = ZAtNmK - float4(ZAtK.y, ZAtK.x, ZAtK.w, ZAtK.z);
					// Mult by 'i'
					FAtNmK = float4(-FAtNmK.y, FAtNmK.x, -FAtNmK.w, FAtNmK.z);
					DstUAV[ DstRect.xy + TexelNmK ] = FAtNmK;
				}
			}
		}
		else // K == 0 || K == N/2
		{
			uint CurRowIdx = RowIdx;
			for (; CurRowIdx < NumRows; CurRowIdx += ThreadsPerGroup)
			{
				uint2 TexelK   = Coord(uint2(K,   CurRowIdx), bIsHorizontal);
				float4 ZAtK    = SrcSRV[ SrcRect.xy + TexelK ]; 
				
				uint2 Texel2 = ( K == 0 ) ?  Coord(uint2(N,   CurRowIdx), bIsHorizontal) : Coord(uint2(N + 1,   CurRowIdx), bIsHorizontal);
				
				float4 ZAt2 = SrcSRV[ SrcRect.xy + Texel2 ]; 
				DstUAV[DstRect.xy + TexelK] = float4(ZAtK.x, ZAt2.x, ZAtK.z, ZAt2.z);
			}
		}

	}

}

#endif //INCLUDE_PACK_TWOFORONE_FFT_PASS

#ifdef INCLUDE_COPY_WINDOW


// Input SRV: 
Texture2DType   SrcSRV;
RWTexture2DType DstUAV;

uint4 DstRect;
uint4 SrcRect;
#define X_THREAD_COUNT 1
#define Y_THREAD_COUNT 32
[numthreads(X_THREAD_COUNT, Y_THREAD_COUNT, 1)]
void CopyWindowCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{

	const bool bModifyInput  = (BrightPixelGain.y > BrightPixelGain.x);

	const uint XTile  = GroupID.x;
	const uint YTile  = GroupID.y;

	const uint XThreadId = GroupThreadID.x;
	const uint YThreadId = GroupThreadID.y;

	uint2 Pixel = uint2(XTile * X_THREAD_COUNT + XThreadId, YTile * Y_THREAD_COUNT + YThreadId);
	
	// Need to window the read and write
	float4 SrcValue = WindowRead(SrcRect.xy + Pixel, SrcRect, SrcSRV);

	if (bModifyInput)
	{
		FilterPixel(BrightPixelGain, SrcValue);
	}

	uint2 Texel = DstRect.xy + Pixel;

	// not in window
	bool bNotIn = !(Texel.x < DstRect.z) ||
		           (Texel.x < DstRect.x) ||
				  !(Texel.y < DstRect.w) ||
				   (Texel.y < DstRect.y);
	if (!bNotIn)
	{
		DstUAV[Texel] = SrcValue;
	}

	//DstUAV[DstRect.xy + Pixel] = SrcValue; 
	
}

#endif //INCLUDE_COPY_WINDOW

#ifdef INCLUDE_COMPLEX_MULTIPLY_IMAGES

// Input SRV: 
Texture2DType   SrcSRV;
Texture2DType   KnlSRV;
RWTexture2DType DstUAV;

uint4 SrcRect;
uint DataLayout;  // 1 for horizontal

#define NUMTHREADS_PER_COL 32

[numthreads(NUMTHREADS_PER_COL, 1, 1)]
void ComplexMultiplyImagesCS(uint3 GroupID : SV_GroupID, uint3 GroupThreadID : SV_GroupThreadID )
{
	
	const uint ThreadCount = NUMTHREADS_PER_COL;

	// Are our scan-lines horizontal?
	const bool bIsHorizontal = (DataLayout == 1);
	

	const uint NumScanlines   = (bIsHorizontal) ? SrcRect.w - SrcRect.y  :  SrcRect.z - SrcRect.x;
	const uint ScanlineLength = (bIsHorizontal) ? SrcRect.z - SrcRect.x  :  SrcRect.w - SrcRect.y;

	
	
	float4 InvNorm;
	// Get Values needed for normalization
	{

		const bool bIsGAGroup = ( (2 *  GroupID.z) > (NumScanlines - 2) );
		Complex Norm[2];
		
		
		const bool bUseAlpha = true;

		GetKernelSum(KnlSRV, bIsHorizontal, NumScanlines, Norm);

		// redSum = Norm[0].x;  greenSum = Norm[0].y; blueSum = Norm[1].x; alphaSum = Norm[1].y

		// We normalize each RGBA channel independently.
		InvNorm = float4(rcp((bIsGAGroup) ? Norm[0].y : Norm[0].x).xx, rcp((bIsGAGroup) ? Norm[1].y : Norm[1].x).xx);
	}


	// All the threads in this group operate on this column.

	const uint ScanLineIdx  = GroupID.z;
	//const bool bIsGAGroup = ( (2 * ScanLineIdx) > (NumScanlines - 2) );

	// This thread is responsible for the element (ElementIdx, ScanIdx) 
	const uint ElementIdx = GroupThreadID.x;

	// Write to the same size/location window.
	const uint4 DstRect = SrcRect;
	
	if (bIsHorizontal)
	{
		uint Loc = ElementIdx;
		for ( ; Loc < ScanlineLength; Loc += ThreadCount)
		{
			uint2 Pixel = uint2( Loc, ScanLineIdx) + SrcRect.xy;
			float4 SrcValue = SrcSRV[Pixel];
			SrcValue *= InvNorm;
			float4 KnlValue = KnlSRV[Pixel];
			
			float2 RorG = ComplexMult(SrcValue.xy, KnlValue.xy);
			float2 BorA = ComplexMult(SrcValue.zw, KnlValue.zw);

			DstUAV[Pixel] = float4(RorG, BorA);
			
		}
	}
	else
	{
		uint Loc = ElementIdx;
		for ( ; Loc < ScanlineLength; Loc += ThreadCount)
		{
			uint2 Pixel = uint2(ScanLineIdx, Loc) + SrcRect.xy;
			float4 SrcValue = SrcSRV[Pixel];
			SrcValue *= InvNorm;
			float4 KnlValue = KnlSRV[Pixel];
			
			float2 RorG = ComplexMult(SrcValue.xy, KnlValue.xy);
			float2 BorA = ComplexMult(SrcValue.zw, KnlValue.zw);

			DstUAV[Pixel] = float4(RorG, BorA);
		}
	}
}

#endif // INCLUDE_COMPLEX_MULTIPLY_IMAGES
