// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GPUFastFourierTransform2DCore.usf: Core 2D Fast Fourier Transform Code 
=============================================================================*/

#pragma once

// This file requires the two values to be defined.
// SCAN_LINE_LENGTH must be a power of RADIX and RADIX in turn must be a power of two 
// Need define:  RADIX and SCAN_LINE_LENGTH

// The core FFT functionality

#include "GPUFastFourierTransformCore.ush"


// ---------------------------------------------------------------------------------------------------------------------------------------
//				For the 2D FFT - Utilities used copying data between main memory and local registers where each thread of the FFT works.
// --------------------------------------------------------------------------------------------------------------------------------------- 

// The Src and Dst buffers containt two types of data: 
//      A real 4 channel color (r,g,b,a)  or  two complex numbers  float4.{xy, zw} = (Complex, Complex) 

#define SRCTYPE  float4
#define DSTTYPE  float4

#define Texture2DType    Texture2D<SRCTYPE>
#define RWTexture2DType  RWTexture2D<DSTTYPE>

// Input SRV: 
Texture2DType SrcTexture;

// Output: Real and Imaginary Parts (UAV)
RWTexture2DType DstTexture;



// Utility to replace any NaNs with zeros.
void ScrubNANs(inout Complex LocalBuffer[2][RADIX])
{

	UNROLL
	for (uint r = 0; r < RADIX; ++r)
	{	
	
		LocalBuffer[0][r] = -min(-LocalBuffer[0][r], Complex(0,0) ); 
		LocalBuffer[1][r] = -min(-LocalBuffer[1][r], Complex(0,0) ); 
	}
}


// Copy Data from main memory (src texture) to local 

void CopyDataSrcToLocal(inout Complex LocalBuffer[2][RADIX], bool bIsHorizontal, uint ScanIdx, uint Loc, uint Stride )
{
	if (bIsHorizontal) 
	{
		uint2 Pixel = uint2(Loc, ScanIdx);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.x += Stride)
		{
			float4 SrcValue = SrcTexture[Pixel];
			LocalBuffer[0][ i ] = SrcValue.xy;
			LocalBuffer[1][ i ] = SrcValue.zw;
		}
	}
	else 
	{
		uint2 Pixel = uint2(ScanIdx, Loc);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.y += Stride)
		{
			float4 SrcValue = SrcTexture[Pixel];
			LocalBuffer[0][ i ] = SrcValue.xy;
			LocalBuffer[1][ i ] = SrcValue.zw;
		}
	}
}

// Copy Data back to main memory (dst)

void CopyDataLocalToDst(in Complex LocalBuffer[2][RADIX], bool bIsHorizontal, uint ScanIdx, uint Loc, uint Stride)
{

	if(bIsHorizontal)
	{
		uint2 Pixel = uint2(Loc, ScanIdx);
		UNROLL
		for (uint r = 0; r < RADIX; ++r, Pixel.x += Stride)
		{
			float4 DstValue;
			DstValue.xy = LocalBuffer[0][r];
			DstValue.zw = LocalBuffer[1][r];
			DstTexture[Pixel] = DstValue;
		}
	}
	else
	{
		uint2 Pixel = uint2(ScanIdx, Loc);
		UNROLL
		for (uint r = 0; r < RADIX; ++r, Pixel.y += Stride)
		{
			float4 DstValue;
			DstValue.xy = LocalBuffer[0][r];
			DstValue.zw = LocalBuffer[1][r];
			DstTexture[Pixel] = DstValue;
		}
	}
}

// Copy Data from main memory (src texture) to local buffer
// Loads zero values for areas outside the window.

void CopyDataSrcWindowToLocal(inout Complex LocalBuffer[2][RADIX], bool bIsHorizontal, in uint ScanIdx, uint Loc, uint Stride, uint4 Window)
{
	{ for (uint i = 0; i < RADIX; ++i) LocalBuffer[0][ i ] = float2(0.f, 0.f); }
	{ for (uint i = 0; i < RADIX; ++i) LocalBuffer[1][ i ] = float2(0.f, 0.f); }
	
	if (bIsHorizontal) 
	{
		// offset for window start
		uint2 Pixel = uint2(Loc, ScanIdx) + Window.xy;
		UNROLL
		for (uint i = 0; i < RADIX ; ++i, Pixel.x += Stride)
		{	
			bool InWindow = Pixel.x < Window.z; 
			if (InWindow)
			{ 
				float4 SrcValue = SrcTexture[Pixel];
				LocalBuffer[0][ i ] = SrcValue.xy;
				LocalBuffer[1][ i ] = SrcValue.zw;
			}
			else
			{
				LocalBuffer[0][ i ] = 0.0;
				LocalBuffer[1][ i ] = 0.0;
			}

		}
	}
	else 
	{
		// offset for window start
		uint2 Pixel = uint2(ScanIdx, Loc) + Window.xy;
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.y += Stride)
		{
			bool InWindow = Pixel.y < Window.w;
			if (InWindow)
			{ 
				float4 SrcValue = SrcTexture[Pixel];
				LocalBuffer[0][ i ] = SrcValue.xy;
				LocalBuffer[1][ i ] = SrcValue.zw;
			}
			else
			{
				LocalBuffer[0][ i ] = 0.0;
				LocalBuffer[1][ i ] = 0.0;
			}
		}
	}
}


// Copy Data from main memory (src texture) to local buffer
// Loads zero values for areas outside the window.

void CopyDataSrcWindowToLocal(inout Complex LocalBuffer[2][RADIX], bool bIsHorizontal, uint ScanIdx, uint Loc, uint Stride, uint2 WindowMin, uint2 WindowMax )
{
	{ for (uint i = 0; i < RADIX; ++i) LocalBuffer[0][ i ] = float2(0.f, 0.f); }
	{ for (uint i = 0; i < RADIX; ++i) LocalBuffer[1][ i ] = float2(0.f, 0.f); }

	if (bIsHorizontal) 
	{
		uint2 Pixel = uint2(Loc, ScanIdx) + uint2(WindowMin.x, 0);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.x += Stride)
		{	
			bool InWindow = Pixel.x < WindowMax.x; 
			if (InWindow)
			{ 
				float4 SrcValue = SrcTexture[Pixel];
				LocalBuffer[0][ i ] = SrcValue.xy;
				LocalBuffer[1][ i ] = SrcValue.zw;
			}	
			else
			{
				LocalBuffer[0][ i ] = 0.0;
				LocalBuffer[1][ i ] = 0.0;
			}	
		}
	}
	else 
	{
		uint2 Pixel = uint2(ScanIdx, Loc) + uint2(0, WindowMin.y);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.y += Stride)
		{
			bool InWindow = Pixel.y < WindowMax.y;
			if (InWindow)
			{	
				float4 SrcValue = SrcTexture[Pixel];
				LocalBuffer[0][ i ] = SrcValue.xy;
				LocalBuffer[1][ i ] = SrcValue.zw;
			}
			else
			{
				LocalBuffer[0][ i ] = 0.0;
				LocalBuffer[1][ i ] = 0.0;
			}
		}
	}
}

// Copy windowed Data back to main memory aligned with ROIRect.xy

void CopyDataLocalToDstWindow(in Complex LocalBuffer[2][RADIX], bool bIsHorizontal, in uint ScanIdx, uint Loc, uint Stride, uint4 ROIRect)
{

	if(bIsHorizontal)
	{
		
		uint2 Pixel = uint2(Loc + ROIRect.x, ScanIdx + ROIRect.y);

		UNROLL_N(RADIX)
		for (uint r = 0; r < RADIX && Pixel.x < ROIRect.z; ++r, Pixel.x += Stride)
		{
			float4 DstValue;
			DstValue.xy = LocalBuffer[0][r];
			DstValue.zw = LocalBuffer[1][r];

			DstTexture[Pixel] = DstValue;
		}
	}
	else
	{
		uint2 Pixel = uint2(ScanIdx + ROIRect.x, Loc + ROIRect.y);

		UNROLL_N(RADIX)
		for (uint r = 0; r < RADIX && Pixel.y < ROIRect.w; ++r, Pixel.y += Stride)
		{
			float4 DstValue;
			DstValue.xy = LocalBuffer[0][r];
			DstValue.zw = LocalBuffer[1][r];

			DstTexture[Pixel] = DstValue;
		}
	}
}


// Copy windowed Data back to main aligned with 0,0

void CopyDataLocalToDstWindow(in Complex LocalBuffer[2][RADIX], bool bIsHorizontal, uint ScanIdx, uint Loc, uint Stride, uint2 Extent)
{
	uint4 ROIRect = uint4(0, 0, Extent.x, Extent.y);
	CopyDataLocalToDstWindow(LocalBuffer, bIsHorizontal, ScanIdx, Loc, Stride, ROIRect);
}


// ---------------------------------------------------------------------------------------------------------------------------------------
//				For the 2D FFT - Specialized Utilities that manage the main memory layout of "two-for-one" results.
//                               when transforming 4 channels (r.g.b.a) as two complex signals (r+ig, b+ia)
//                               and unpacking the 4 resulting 1/2 length complex transforms R, G, B, A 
// --------------------------------------------------------------------------------------------------------------------------------------- 


// Writes the (thread local) data from a 2-for-1 transform (transform of real f and g packed as f+i*g) to main memory, by first splitting F and G
// where 'F' is the transform of 'f' and 'G' is the transform of 'g.'
//
//	NB: This requires the buffer be of length SignalLength + 2. 
//	NB: F_o, G_o, F_N/2 and G_N/2 will be real.. All other coefficients are complex
//
//  here N = SignalLength.  The resulting data layout will be
//
// float2 value: {F_o, 0}, {F_1}, {F_2},..,{F_{N/2-1}}, {F_N/2, 0}, {G_{N/2 +1}},  {G_{N/2 +2}} . .{ G_{N-2}},   {G_{N-1}},    {G_o, 0}, {G_N/2,0} 
//  offset:          0       , 1    , 2    ,..., N/2-1,       N/2,       N/2 +1,      N/2 +2, ..        N-2,        N-1,         N,         N + 1
//
void WriteTwoForOneFrequencyData(in bool bIsHorizontal, inout Complex LocalBuffer[2][RADIX], uint ScanIdx, uint Loc, uint Stride, uint N)
{

	FFTMemoryBarrier();

	// Decompose the transforms.  Note '0' and 'N/2' offsets will still be mixed, and have to be explicitly dealt with below. 

	SplitTwoForOne(LocalBuffer, Loc, Stride, N);

	const bool bIsFirstElement = (Loc == 0);
	const uint Non2 =  N / 2;
	if (bIsHorizontal)
	{
		uint2 Pixel = uint2(Loc, ScanIdx);
		float4 DstValue;
		UNROLL
		for (uint r = 0; r < RADIX ; ++r, Pixel.x += Stride)
		{
			
			
			DstValue.xy = LocalBuffer[ 0 ][ r ];
			DstValue.zw = LocalBuffer[ 1 ][ r ];
			DstTexture[Pixel] = DstValue;

			// The N/2 element holds F_N/2 + I G_N/2
			// Write F_N/2 into this column, and G_N/2 into the the last column 
			if (Pixel.x == Non2)
			{
				DstTexture[Pixel]                 = float4(DstValue.x, 0.f, DstValue.z, 0.f);
				DstTexture[uint2(N + 1, Pixel.y)] = float4(DstValue.y, 0.f, DstValue.w, 0.f);
			}
		}

		// First element holds F_o + iG_o.  
		// Write Go into the second to last column. (this is the same as G_N)
		if (bIsFirstElement)
		{
			DstValue.xy = LocalBuffer[ 0 ][ 0 ];
			DstValue.zw = LocalBuffer[ 1 ][ 0 ];

			DstTexture[uint2(0, Pixel.y)]  = float4(DstValue.x, 0.f, DstValue.z, 0.f); // F_o 
			DstTexture[uint2(N, Pixel.y)]  = float4(DstValue.y, 0.f, DstValue.w, 0.f); // G_o
		}
	}
	else
	{
		uint2 Pixel = uint2(ScanIdx, Loc);
		float4 DstValue;
		UNROLL
		for (uint r = 0; r < RADIX ; ++r, Pixel.y += Stride)
		{
			
			DstValue.xy = LocalBuffer[ 0 ][ r ];
			DstValue.zw = LocalBuffer[ 1 ][ r ];
			DstTexture[Pixel] = DstValue;

			
			// The N/2 element holds F_N/2 + I G_N/2
			// Write F_N/2 into this column, and G_N/2 into the the last column 
			if (Pixel.y == Non2)
			{
				DstTexture[Pixel]                 = float4(DstValue.x, 0.f, DstValue.z, 0.f);
				DstTexture[uint2(Pixel.x, N + 1)] = float4(DstValue.y, 0.f, DstValue.w, 0.f);
			}
		
		}

		// First element holds F_o + iG_o.  
		// Write Go into the second to last column. (this is the same as G_N)
		if (bIsFirstElement)
		{
			DstValue.xy = LocalBuffer[ 0 ][ 0 ];
			DstValue.zw = LocalBuffer[ 1 ][ 0 ];

			DstTexture[uint2(Pixel.x, 0)]  = float4(DstValue.x, 0.f, DstValue.z, 0.f); // F_o
			DstTexture[uint2(Pixel.x, N)]  = float4(DstValue.y, 0.f, DstValue.w, 0.f); // G_o
		}
	}
}


// The inverse of WriteTwoForOneFrequencyData()
// Reads into local registers, data written in the TwoForOneFrequency layout back into the form consistent with the transform of a single complex signal.

void ReadTwoForOneFrequencyData(bool bIsHorizontal, inout Complex LocalBuffer[2][RADIX], in uint ScanIdx, in uint Loc, in uint Stride, in uint N)
{
	const bool bIsFirstElement = (Loc == 0);
	const uint Non2 =  N / 2;

	if (bIsHorizontal) 
	{
		// last two values
		float4 NValue   = SrcTexture[uint2(N, ScanIdx)];
		float4 NppValue = SrcTexture[uint2(N +1, ScanIdx)];

		uint2 Pixel = uint2(Loc, ScanIdx);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.x += Stride)
		{	
			float4 SrcValue = SrcTexture[Pixel];
			LocalBuffer[ 0 ][ i ] = SrcValue.xy;
			LocalBuffer[ 1 ][ i ] = SrcValue.zw;

			if ( Pixel.x ==  Non2)
			{
				// local buffer will be pure real with F_N/2,  need to add I * G_N/2 (G_N/2 is real ie float2(G_r, 0))
				float4 TmpValue = NppValue; // will be (#,0,#,0)
				LocalBuffer[ 0 ][ i ] += NppValue.yx;
				LocalBuffer[ 1 ][ i ] += NppValue.wz;
			}

		}

		if (bIsFirstElement)
		{
			float4 LastSrcValue = SrcTexture[uint2(N, Pixel.y)]; // will be (#,0,#,0)
			LocalBuffer[ 0 ][ 0 ] += NValue.yx; 
			LocalBuffer[ 1 ][ 0 ] += NValue.wz;
		}

	}
	else 
	{
		// last two values
	    float4 NValue   = SrcTexture[uint2(ScanIdx, N)];
		float4 NppValue = SrcTexture[uint2(ScanIdx, N + 1)];
		
		uint2 Pixel = uint2(ScanIdx, Loc);
		UNROLL
		for (uint i = 0; i < RADIX; ++i, Pixel.y += Stride)
		{
			float4 SrcValue = SrcTexture[Pixel];
			LocalBuffer[ 0 ][ i ] = SrcValue.xy;
			LocalBuffer[ 1 ][ i ] = SrcValue.zw;

			if ( Pixel.y ==  Non2)
			{
				// local buffer will be pure real with F_N/2,  need to add IG_N/2
				LocalBuffer[ 0 ][ i ] += NppValue.yx;
				LocalBuffer[ 1 ][ i ] += NppValue.wz;
			}
		}
	
		if (bIsFirstElement)
		{
			LocalBuffer[ 0 ][ 0 ] += NValue.yx; 
			LocalBuffer[ 1 ][ 0 ] += NValue.wz;
		}
	}

	

	// Combine the transforms of the two real signals (F,G) as Z = F + I G
	MergeTwoForOne(LocalBuffer, Loc, Stride, N);
	
	// Done with the group shared memory that was used in the merge
	FFTMemoryBarrier();
	
}