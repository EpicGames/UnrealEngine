// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GPUFastFourierTransformCore.usf: Core Fast Fourier Transform Code
=============================================================================*/

//------------------------------------------------------- RECOMPILE HASH
#pragma message("UESHADERMETADATA_VERSION B96BD6F6-825A-4B16-AA27-96502C97998C")


// This file requires the two values to be defined.
// SCAN_LINE_LENGTH must be a power of RADIX and RADIX in turn must be a power of two 
// Need define:  RADIX and SCAN_LINE_LENGTH


#pragma once

#include "Common.ush"

#ifndef TWO_PI
	#define TWO_PI 	 2.0f * PI
#endif


#define FFTMemoryBarrier() GroupMemoryBarrierWithGroupSync()


// Use floating point number representation.
// Complex = x + iy, where x, y are both real numbers (floats) and i = sqrt(-1)

#define Complex  float2

// Complex Multiplication using Complex as a complex number
// The full complex arithmetic is defined as:
// Real(A) = A.x,                          Image(A) = A.y
// Real(A + B) = A.x + B.x;                Imag(A + B) = A.y + B.y
// Real(A * B)  = A.x * B.x - A.y * B.y;   Imag(A * B) = A.x * B.y + B.x * A.y

Complex ComplexMult(in Complex A, in Complex B)
{
	return Complex(A.x * B.x - A.y * B.y, A.x * B.y + B.x * A.y);
}

Complex ComplexConjugate(in Complex Z)
{
	return Complex(Z.x, -Z.y);
}



// ------------------------------------------------
// The Definition of the Fourier Transform:
// ------------------------------------------------
// There is ambiguity in the definition of the Fourier Transform
// so we explicitly state our definition.
// 
//  Given N data samples f_n (gernerally complex) with n = [0, N-1] 
//  the transform produces N values  F_k with k = [0, N-1].
//  
// These values (generally complex) are defined as
//
// F_k = Sum[f_n * Exp[i * n *k * 2 * Pi / N], {n, 0, N-1}]
//
//  This may be invereted by computing
// 
// f_n = (1/N) Sum[F_k * Exp[-i * n * k * 2 * Pi / N], {k, 0, N-1}]
//
// In both these sums 'i' is the complex number sqrt[-1], and
// Exp[ i x] = Cos[x] + i Sin[x].
//
// The actual Fast Fourier Transform (FFT) exploits symetries in the 
// trigonometric functions the allow us to compute the N different values
// in the transform in N*Log(N) time (not N*N as you direct implimentaiton of
// the sum would give)
// 
// Note: the 'normalization' term (1/N) in the inverse. 
//
// Also Note :
//     If the complex input signal z_n = x_n + i y_n is in fact two separate real signals x_n, y_n  
//     then the FFT of each signal can be extracted from Z_k = FFT(z).
//     X_k = FFT(x) =    (Z_k + Conjugate(Z_{N-k})) / 2
//     Y_k = FFT(y) = -i (Z_k - Conjugate(Z_{N-k})) / 2
//
//     In general the transform of real data f_n has the symmetry 
//     F_k = Conjugate(F_{N-k})

#define NORMALIZE_ORDER  1


float ForwardScale(uint ArrayLength)
{	
#if NORMALIZE_ORDER  == 1
	return float(1); 
#else
	return (float(1) / float(ArrayLength) );
#endif
}

float InverseScale(uint ArrayLength)
{
#if NORMALIZE_ORDER == 1
	return ( float(1) / float(ArrayLength) );
#else
	return float(1);
#endif
}

// ------------------------------------------------
// FFT for different Radix size:
// ------------------------------------------------
// Specialized code for doing the FFT or inverse FFT on arrays of size 2, 4, or 8 elements.
// Note: because these small transforms are used as building blocks for transforming longer
// arrays, the normalization term is omitted from the small inverse transforms. 
//
 
void Radix2FFT(in bool bIsForward, inout Complex V0, inout Complex V1)
{
#if 0
	// Reference.

	Complex Tmp = V1;
	V1 = V0 - Tmp;
	V0 = V0 + Tmp;
#endif 

	V0 = V0 + V1;
	V1 = V0 - V1 - V1;
}

void Radix2FFT(in bool bIsForward, inout Complex V[2])
{
	V[0] = V[0] + V[1];
	V[1] = V[0] - V[1] - V[1];
}

void Radix3FFT(in bool bIsForward, inout Complex V[3])
{
	Complex Tmp0 = V[0] + V[1] + V[2];

	// exp( i 2 Pi / 3) = Cos(2 Pi / 3) + i Sin( 2 Pi / 3)
	Complex CS2PiOn3 = Complex(-0.5, 0.5 * sqrt(3.f));
	
	// exp(i 4 Pi / 3) = Cos(4 Pi / 3) + i Sin(4 Pi / 3)
	Complex CS4PiOn3 = Complex(-0.5, -0.5 * sqrt(3.f));
	
	if (bIsForward)
	{
		Complex Tmp1 = V[0] + ComplexMult(CS2PiOn3, V[1]) + ComplexMult(CS4PiOn3, V[2]);
		V[2] = V[0] + ComplexMult(CS4PiOn3, V[1]) + ComplexMult(CS2PiOn3, V[2]);
		V[1] = Tmp1;
		V[0] = Tmp0;
	}
	else 
	{
		Complex Tmp1 = V[0] + ComplexMult(CS4PiOn3, V[1]) + ComplexMult(CS2PiOn3, V[2]);
		V[2] = V[0] + ComplexMult(CS2PiOn3, V[1]) + ComplexMult(CS4PiOn3, V[2]);
		V[1] = Tmp1;
		V[0] = Tmp0;
	}
}


void Radix4FFT(in bool bIsForward, inout Complex V0, inout Complex V1, inout Complex V2, inout Complex V3)
{
	// The even and odd transforms
	Radix2FFT(bIsForward, V0, V2); 
	Radix2FFT(bIsForward, V1, V3); 

	// The butterfly merge of the even and odd transforms
	Complex Tmp;
	Complex TmpV1 = V1;
	 
	if (bIsForward) {
		// Complex(0, 1) * V3
		Tmp = Complex(-V3.y, V3.x);
	}
	else 
	{
		// Complex(0, -1) * V3
		Tmp = Complex(V3.y, -V3.x);
	}


#if 0
	
	// Reference.   

	Complex Rslt[4];
	Rslt[0] = V0 + V1;
	Rslt[1] = V2 + Tmp;
	Rslt[2] = V0 - V1;
	Rslt[3] = V2 - Tmp;

	V0 = Rslt[0]; 
	V1 = Rslt[1];
	V2 = Rslt[2];
	V3 = Rslt[3];

#endif
	
	V0 = V0 + TmpV1;
	V1 = V2 + Tmp;
	V3 = V2 - Tmp;
	V2 = V0 - TmpV1 - TmpV1;
	
}


void Radix4FFT(in bool bIsForward, inout Complex V[4])
{
	// The even and odd transforms
	V[0] = V[0] + V[2];
	V[1] = V[1] + V[3];
	V[2] = V[0] - V[2] - V[2];
	V[3] = V[1] - V[3] - V[3];

	// The butterfly merge of the even and odd transforms
	Complex Tmp;
	Complex TmpV1 = V[1];

	if (bIsForward) {
		// Complex(0, 1) * V3
		Tmp = Complex(-V[3].y, V[3].x);
	}
	else 
	{
		// Complex(0, -1) * V3
		Tmp = Complex(V[3].y, -V[3].x);
	}
	
	V[0] = V[0] + TmpV1;
	V[1] = V[2] + Tmp;
	V[3] = V[2] - Tmp;
	V[2] = V[0] - TmpV1 - TmpV1;
	
}


void Radix8FFT(in bool bIsForward, inout Complex V0, inout Complex V1, inout Complex V2, inout Complex V3, inout Complex V4, inout Complex V5, inout Complex V6, inout Complex V7)
{
	// The even and odd transforms
	Radix4FFT(bIsForward, V0, V2, V4, V6);
	Radix4FFT(bIsForward, V1, V3, V5, V7);

	Complex Twiddle;
	// 0.7071067811865475 = 1/sqrt(2)
	float InvSqrtTwo = float(1.f) / sqrt(2.f);
	if (bIsForward)
	{
		Twiddle = Complex(InvSqrtTwo, InvSqrtTwo);
	}
	else
	{
		Twiddle = Complex(InvSqrtTwo, -InvSqrtTwo);
	}



	Complex Rslt[8];
	Complex Tmp = ComplexMult(Twiddle, V3);

	Rslt[0] = V0 + V1;
	Rslt[4] = V0 - V1;

	Rslt[1] = V2 + Tmp;
	Rslt[5] = V2 - Tmp;

	if (bIsForward)
	{
		// V4 + i V5
		Rslt[2] = Complex(V4.x - V5.y, V4.y + V5.x);
		// V4 - i V5
		Rslt[6] = Complex(V4.x + V5.y, V4.y - V5.x);
	}
	else
	{
		// V4 - iV5
		Rslt[2] = Complex(V4.x + V5.y, V4.y - V5.x);
		// V4 + iV5
		Rslt[6] = Complex(V4.x - V5.y, V4.y + V5.x);
	}	

	Twiddle.x = -Twiddle.x;
	Tmp = ComplexMult(Twiddle, V7);

	Rslt[3] = V6 + Tmp;
	Rslt[7] = V6 - Tmp;

	V0 = Rslt[0];
	V1 = Rslt[1];
	V2 = Rslt[2];
	V3 = Rslt[3];
	V4 = Rslt[4];
	V5 = Rslt[5];
	V6 = Rslt[6];
	V7 = Rslt[7];
}


// Wrapper to select the correct radix FFT 
// at compile time.
//
// Radix in [2, 4, 8]

void RadixFFT(in bool bIsForward, inout Complex v[RADIX])
{
#if (RADIX == 2)
	//Radix2FFT(bIsForward, v[0], v[1]);
	Radix2FFT(bIsForward, v);
#endif

#if (RADIX == 3)
	Radix3FFT(bIsForward, v);
#endif

#if (RADIX == 4)
//	Radix4FFT(bIsForward, v[0], v[1], v[2], v[3]);
	Radix4FFT(bIsForward, v);
#endif

#if (RADIX == 8)
	Radix8FFT(bIsForward, v[0], v[1], v[2], v[3], v[4], v[5], v[6], v[7]);
#endif

#if (RADIX != 2 && RADIX != 4 && RADIX !=8 && RADIX != 3)
	#error "Undefined radix length!"
#endif
}

// Utility: As a function of j, 
// returns Ns contiguous values, then skips R*Ns values, then the next Ns values, etc
// (e.g. R = 3, Ns = 2: 0, 1, 6, 7, 12, 13..)
// (e.g. R = 2, Ns = 4: 0, 1, 2, 3, 8, 9, 10, 11,..)

uint Expand(in uint j, in uint Ns, in uint R) {
	return (j / Ns) * Ns * R + (j % Ns);
}


///--------------------------------------------------------------------------
// Performs a single pass Stockham FFT using group shared memory.
// 
// Inspired by 
// "High Performance Discrete Fourier Transforms on Graphics Processors" - Naga K Govindaraju et al, 2008
// Published in SC'08: Proceedings of the 2008 ACM/IEEE Conference on Supercomputing

// One dimensional arrays of data (e.g. scanlines) are interperated as blocks of row-major data, 
// the columns of which are independently transformed with one thread group per column.
//  
// NB: it is required that the a single column of Complex data fits in group shared memory.
// 
// NUMTHREADSX : Number of threads in a thread group.


// Multiple threads in a single thread group transform an array of lenght 
// 'ArrayLength' using the group shared memory as temporary scratch space.
// #define RADIX; // this should come from the CPU
// #define NUMTHREADSX; // this should come from the CPU  . BlockHeight / RADIX = NUMTHREADSX

// Group shared memory limited to 32k.  We need to split this over 2 buffers.
// 32k * (1024 byte / k) * (1 float / 4 byte ) = 8192 floats - or 4096 Complex<float> 

// This will allow for transforms of lenght 4096 to be done in a single dispatch.




#if SCAN_LINE_LENGTH > 4096
	#define SHARED_MEMORY_BUFFER_COUNT 1  // Testing shows that in general having 2 buffers is faster: the overhead of additional syncs.
#else
	#define SHARED_MEMORY_BUFFER_COUNT 1
#endif


#if SHARED_MEMORY_BUFFER_COUNT==2
groupshared float SharedReal[ SCAN_LINE_LENGTH ];
groupshared float SharedImag[ SCAN_LINE_LENGTH ];


void CopyLocalToGroupShared(in Complex Local[RADIX], in uint Head, in uint Stride)
{
	{
		for (uint r = 0, i = Head; r < RADIX; ++r, i += Stride)
		{
			SharedReal[ i ] = Local[ r ].x;
		}
	}
	 
	{
		for (uint r = 0, i = Head; r < RADIX; ++r, i += Stride)
		{
			SharedImag[ i ] = Local[ r ].y;
		}
	}
}

void CopyLocalToGroupSharedWSync(in Complex Local[RADIX], in uint Head, in uint Stride)
{
	CopyLocalToGroupShared(Local, Head, Stride);

	FFTMemoryBarrier();
}

void CopyGroupSharedToLocal(inout Complex Local[RADIX], in uint Head, in uint Stride)
{
	{
 		for (uint r = 0, i = Head; r < RADIX; ++r, i += Stride) 
		{
			Local[ r ].x = SharedReal[ i ];
		}
	}

	{
		for (uint r = 0, i = Head; r < RADIX; ++r, i += Stride) 
		{
			Local[ r ].y = SharedImag[ i ];
		}
	}
}



// Exchange data with other threads by affecting a transpose
// This accounts for about 1/3rd  of the total time. 

void TransposeData(inout Complex Local[RADIX], uint AHead, uint AStride, uint BHead, uint BStride)
{
	CopyLocalToGroupSharedWSync(Local, AHead, AStride);  // Does a memory barrier

	CopyGroupSharedToLocal(Local, BHead, BStride);
} 

#else // SHARED_MEMORY_BUFFER_COUNT == 1 

groupshared float SharedReal[ 2 * SCAN_LINE_LENGTH ];
#define NUM_BANKS 32

void CopyLocalXToGroupShared(in Complex Local[RADIX], in uint Head, in uint Stride, in uint BankSkip)
{
	uint i = Head;
	UNROLL for (uint r = 0; r < RADIX; ++r, i += Stride)
	{
		uint j = i + (i / NUM_BANKS) * BankSkip;
		SharedReal[ j ] = Local[ r ].x;
	}
}

void CopyLocalYToGroupShared(in Complex Local[RADIX], in uint Head, in uint Stride, in uint BankSkip)
{
	uint i = Head;
	UNROLL for (uint r = 0; r < RADIX; ++r, i += Stride)
	{
		uint j = i + (i / NUM_BANKS) * BankSkip;
		SharedReal[ j ] = Local[ r ].y;
	}
}


void CopyGroupSharedToLocalX(inout Complex Local[RADIX], in uint Head, in uint Stride, in uint BankSkip)
{
	uint i = Head;
	UNROLL for (uint r = 0; r < RADIX; ++r, i += Stride) 
	{
		uint j = i + (i / NUM_BANKS) * BankSkip;
		Local[ r ].x = SharedReal[ j ];
	}

}


void CopyGroupSharedToLocalY(inout Complex Local[RADIX], in uint Head, in uint Stride, in uint BankSkip)
{
	uint i = Head;
	UNROLL for (uint r = 0; r < RADIX; ++r, i += Stride) 
	{
		uint j = i + (i / NUM_BANKS) * BankSkip;
		Local[ r ].y = SharedReal[ j ];
	}

}

void CopyLocalXToGroupShared(in Complex Local[RADIX], in uint Head, in uint Stride)
{
	CopyLocalXToGroupShared(Local, Head, Stride, 0);
}

void CopyLocalYToGroupShared(in Complex Local[RADIX], in uint Head, in uint Stride)
{
	CopyLocalYToGroupShared(Local, Head, Stride, 0);
}


void CopyGroupSharedToLocalX(inout Complex Local[RADIX], in uint Head, in uint Stride)
{
	CopyGroupSharedToLocalX(Local, Head, Stride, 0);
}	


void CopyGroupSharedToLocalY(inout Complex Local[RADIX], in uint Head, in uint Stride)
{
	CopyGroupSharedToLocalY(Local, Head, Stride, 0);
}


// Exchange data with other threads by affecting a transpose 
void TransposeData(inout Complex Local[RADIX], uint AHead, uint AStride, uint BHead, uint BStride)
{
	uint BankSkip = (AStride < NUM_BANKS) ? AStride : 0;

	CopyLocalXToGroupShared(Local, AHead, AStride, BankSkip);
	
	FFTMemoryBarrier();

	CopyGroupSharedToLocalX(Local, BHead, BStride, BankSkip);

	FFTMemoryBarrier();

	CopyLocalYToGroupShared(Local, AHead, AStride, BankSkip);

	FFTMemoryBarrier();

	CopyGroupSharedToLocalY(Local, BHead, BStride, BankSkip);

} 

#endif // SHARED_MEMORY_BUFFER_COUNT

// This accounts for about 9% of total time
void Butterfly(bool bIsForward, inout Complex Local[RADIX], uint ThreadIdx, uint Length)
{

	float angle =  TWO_PI * float( ThreadIdx % Length) / float(Length * RADIX);
	if (!bIsForward) {
		angle *= -1;
	}

		
	Complex TwiddleInc;
	sincos(angle, TwiddleInc.y, TwiddleInc.x);
	Complex Twiddle = TwiddleInc;
	for (uint r = 1; r < RADIX; ++r) {
		Local[r] = ComplexMult(Twiddle, Local[r]);
		Twiddle  = ComplexMult(Twiddle, TwiddleInc);
	}
}

void ButterflyRadix4(bool bIsForward, inout Complex Local[RADIX], uint ThreadIdx, uint Length)
{

	float AngleEven = TWO_PI * float(ThreadIdx) / float(Length * RADIX);
	float AngleOdd = TWO_PI * float(ThreadIdx+Length) / float(Length * RADIX);
	if (!bIsForward)
	{
		AngleEven *= -1;
		AngleOdd *= -1;
	}

		
	Complex TwiddleIncEven;
	sincos(AngleEven, TwiddleIncEven.y, TwiddleIncEven.x);
	Complex TwiddleIncOdd;
	sincos(AngleOdd, TwiddleIncOdd.y, TwiddleIncOdd.x);
	Complex TwiddleEven = TwiddleIncEven;
	Complex TwiddleOdd = TwiddleIncOdd;
	for (uint r = 2; r < RADIX; r+=2)
	{
		Local[r] = ComplexMult(TwiddleEven, Local[r]);
		TwiddleEven = ComplexMult(TwiddleEven, TwiddleIncEven);
		Local[r+1] = ComplexMult(TwiddleOdd, Local[r+1]);
		TwiddleOdd = ComplexMult(TwiddleOdd, TwiddleIncOdd);
	}
}

void ButterflyRadix2(bool bIsForward, inout Complex Local[RADIX], uint ThreadIdx, uint Length)
{

	float AngleStart = TWO_PI * float(ThreadIdx) / float(Length * RADIX);
	float AngleInc = TWO_PI * float(Length) / float(Length * RADIX);
	if (!bIsForward)
	{
		AngleStart *= -1;
		AngleInc *= -1;
	}

		
	Complex TwiddleInc;
	sincos(AngleInc, TwiddleInc.y, TwiddleInc.x);
	Complex Twiddle;
	sincos(AngleStart, Twiddle.y, Twiddle.x);
	for (uint r = 4; r < RADIX; ++r)
	{
		Local[r] = ComplexMult(Twiddle, Local[r]);
		Twiddle = ComplexMult(Twiddle, TwiddleInc);
	}
}

// Performs a single pass Stockham FFT using group shared memory.
void GroupSharedFFT(in const bool bIsForward, inout Complex Local[RADIX], in const uint ArrayLength, in const uint ThreadIdx)
{
	// Parallelism from writting the 1d array as a 2d row-major array: n = J *m + j.  0 <= j < m,  0 <= J < M
	// f(n) = f(J*m + j) = f(J,j) transforms to
	// F(k) = F(w*M + W) = f(w, W)                                                    0 <= w < m,  0 <= K < M 
	//
	//  F(k) = sum_{n=0}^{N-1} \exp\left( 2\pi i \frac{n k}{N} \right) f(n)
	//
	// can be expressed as
	// 
	//  F(w*m + W) = sum_{j=0}^{m-1} \exp\left( 2 \pi i \frac{w j}{m}\right)  \left[  \exp\left(2\pi i \frac{W j}{m M} \sum_{J=0}^{M-1} \exp(2\pi i \frac{J W}{M} f(J,j) \right]
	
	// Number of rows in 2d row-major layout of an array of lenght ArrayLength

	uint NumCols = ArrayLength / RADIX;


	//uint IdxS = Expand(j, NumCols, RADIX);
	//uint IdxS = (ThreadIdx / NumCols) * ArrayLength + (ThreadIdx % NumCols);
	
	uint IdxS = ThreadIdx;
		
    // uint Ns = 1;
	// (j / Ns) * Ns * R + (j % Ns);
	// Expand(j, Ns, RADIX);
	uint IdxD = ThreadIdx * RADIX;

	// Transform these RADIX values.

	RadixFFT(bIsForward, Local);

	// Exchange data with other threads 
	
	TransposeData(Local, IdxD, 1, IdxS, NumCols);		


	// NumCols = ArrayLength / RADIX: NumCols is an int power of RADIX (ie. 0, 1, 2..)
	uint Ns = RADIX;

#if COMPILER_PSSL
	LOOP
#endif
	for ( ; Ns < NumCols; Ns *= RADIX ) 
	{
		
		Butterfly(bIsForward, Local, ThreadIdx, Ns);

		// IdxD(ThreadIdx) is contiguous blocks of lenght Ns

		IdxD = Expand(ThreadIdx, Ns, RADIX);
		//uint Expand(in uint j, in uint Ns, in uint R) { return (j / Ns) * Ns * R + (j % Ns);}

		// Transform these RADIX values.

		RadixFFT(bIsForward, Local);
		
		// Wait for the rest of the thread group to finish working on this ArrayLength of data. 

		FFTMemoryBarrier();
		
		TransposeData(Local, IdxD, Ns, IdxS, NumCols);
	}
	
#if MIXED_RADIX == 0 || SCAN_LINE_LENGTH == 4096 || SCAN_LINE_LENGTH == 512 || SCAN_LINE_LENGTH == 64 || SCAN_LINE_LENGTH <= 8
	Butterfly(bIsForward, Local, ThreadIdx, Ns);

	// Transform these RADIX values.

	RadixFFT(bIsForward, Local);
#elif SCAN_LINE_LENGTH == 2048 || SCAN_LINE_LENGTH == 256 || SCAN_LINE_LENGTH == 32
	ButterflyRadix4(bIsForward, Local, ThreadIdx, NumCols);
	
	Radix4FFT(bIsForward, Local[0], Local[2], Local[4], Local[6]);
	Radix4FFT(bIsForward, Local[1], Local[3], Local[5], Local[7]);
#elif SCAN_LINE_LENGTH == 1024 || SCAN_LINE_LENGTH == 128 || SCAN_LINE_LENGTH == 16
	ButterflyRadix2(bIsForward, Local, ThreadIdx, NumCols);
	
	Radix2FFT(bIsForward, Local[0], Local[4]);
	Radix2FFT(bIsForward, Local[1], Local[5]);
	Radix2FFT(bIsForward, Local[2], Local[6]);
	Radix2FFT(bIsForward, Local[3], Local[7]);
#else
	#error Unknown SCAN_LINE_LENGTH
#endif
	
	// not needed for the simple case
	
	FFTMemoryBarrier();
}


// Scale two complex signals.
void Scale(inout Complex LocalBuffer[2][RADIX], in float ScaleValue)
{
	// Scale
	{
		for (uint r = 0; r < RADIX; ++r)
		{	
			LocalBuffer[0][r] *= ScaleValue; 
		}
	}

	{
		for (uint r = 0; r < RADIX; ++r)
		{	
			LocalBuffer[1][r] *= ScaleValue; 
		}
	}
}


// Transforming two Complex signals.

void GroupSharedFFT(in bool bIsForward, inout Complex LocalBuffer[2][RADIX], in uint ArrayLength, in uint ThreadIdx)
{
	// Note: The forward and inverse FFT require a 'normalization' scale, such that the normalization scale
	// of the forward times normalization scale of the inverse = 1 / ArrayLenght.
	// ForwardScale(ArrayLength) * InverseScale(ArrayLength) = 1 / ArrayLength;
	
	// Scale Forward
	
	if (bIsForward)
	{
		Scale(LocalBuffer, ForwardScale(ArrayLength));
	}
	else
	{
		Scale(LocalBuffer, InverseScale(ArrayLength));
	}
	
	// Transform each buffer.

	GroupSharedFFT(bIsForward, LocalBuffer[0], ArrayLength, ThreadIdx);
	

	GroupSharedFFT(bIsForward, LocalBuffer[1], ArrayLength, ThreadIdx);

}


// ---------------------------------------------------------------------------------------------------------------------------------------
//				Specialized Utilities used when transforming two real signals as one complex 
//              The "Two-For-One" trick
//              Note these use group-shared memory to detangle the data
// 
//              Transforming 4 channels (r.g.b.a) as two complex signals (r+ig, b+ia)
//              and unpacks the 4 resulting 1/2 length complex transforms R, G, B, A 
// --------------------------------------------------------------------------------------------------------------------------------------- 


// The Two-For-One trick:  FFT two real signals (f,g) of length N 
// are transformed simultaneously by treating them as a single complex signal z.
// z_n = f_n + I * g_n.
// 
// Giving the complex amplitudes (i.e. the transform)  Z_k = F_k + I * G_k.
//
// The transforms of the real signals can be recovered as
//
//  F_k = (1/2) (Z_k + ComplexConjugate(Z_{N-k}))
//  G_k = (-i/2)(Z_k - ComplexConjugate(Z_{N-k}))
//
// Furthermore, the transform of any real signal x_n has the symmetry
//  X_k = ComplexConjugate(X_{N-k})   


// Separate the FFT of two real signals that were processed togehter
// using the two-for-one trick.
//
// On input the data in LocalBuffer across the threads in this group
// holds the transform  of the complex signal z = f + I * g of lenght N.
//
// On return the transform of f and g have been separated and stored
// in the LocalBuffer in a form consistent with the packing the x-signal transform forward, 
// followed by the y-signal transform in reverse.
// 
// {F_o, G_o}, {F_1}, {F_2},..,{F_{N/2-1}}, {F_N/2, G_N/2}, {G_{N/2 +1}},  {G_{N/2 +2}}, .., {G_{N-2}},    {G_{N-1}} 
//     0       , 1    , 2    ,..., N/2-1,       N/2,           N/2 +1,      N/2 +2,  ..,  N/2- (2-N/2),   N/2 - (1-N/2)
// here {} := Complex
#if SHARED_MEMORY_BUFFER_COUNT==2
void SplitTwoForOne(inout Complex LocalBuffer[RADIX], in uint Head, in uint Stride, in uint N)
{

	const uint Non2 = N / 2;	
	
	// On input, Each thread has, in LocalBuffer, the frequencies
	// K = Head + j * Stride;  where j = 0, .., RADIX-1.. Head = ThreadIDx, Stride = NumberOfThreads

	// Write the complex FFT into group shared memory.
	
	CopyLocalToGroupSharedWSync(LocalBuffer, Head, Stride);

	// Construct the transform for the two real signals in the LocalBuffer

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			//
			// K = N/2 - abs(SrcIdx - N/2)

			// DstK = SrcIdx % Non2;
			// N - k
		
			uint NmK = (K > 0) ? ( N - K) : 0;

			// Z_k = LocalBuffer[i]
			// If k < N/2  : Store  F_k = 1/2 (Z_k + Z*_{N-k})
			// If k > N/2  : Compute I*G_k = 1/2 (Z_k - Z*_{N-k})
		
			// Tmp =  {+,-}ComplexConjugate( Z_{N-k})
			Complex Tmp = Complex( SharedReal[NmK], -SharedImag[NmK] );
		
			Tmp *= (K > Non2)? -1 : 1;

			LocalBuffer[i] += Tmp;
		
		}
	}

	{
		for (uint i =0; i < RADIX; ++i) LocalBuffer[ i ] *= 0.5f;
	}
	
	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			// If k > N/2 get G_k from I*G_k:  G_k = -I * (I G_k)
			if (K > Non2) LocalBuffer[i] = ComplexMult(Complex(0, -1), LocalBuffer[i] );
		
			if (K == Non2) 
			{	
				// F_N/2 + I * G_N/2
				LocalBuffer[i] = Complex(SharedReal[Non2], SharedImag[Non2]);
			}
	
		}
	}

	if (Head == 0) 
	{   //                       F_o + I * G_o
		LocalBuffer[0] = Complex(SharedReal[0], SharedImag[0]);
	}		
}

#else // SHARED_MEMORY_COUNT == 1 

void SplitTwoForOne(inout Complex LocalBuffer[RADIX], in uint Head, in uint Stride, in uint N)
{

	const uint Non2 = N / 2;	
	
	// On input, Each thread has, in LocalBuffer, the frequencies
	// K = Head + j * Stride;  where j = 0, .., RADIX-1.. Head = ThreadIDx, Stride = NumberOfThreads

	// Write the complex FFT into group shared memory.
	
	CopyLocalXToGroupShared(LocalBuffer, Head, Stride);
	
	FFTMemoryBarrier();

	// Construct the transform for the two real signals in the LocalBuffer

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			//
			// K = N/2 - abs(SrcIdx - N/2)

			// DstK = SrcIdx % Non2;
			// N - k
		
			uint NmK = (K > 0) ? ( N - K) : 0;

			// Z_k = LocalBuffer[i]
			// If k < N/2  : Store  F_k = 1/2 (Z_k + Z*_{N-k})
			// If k > N/2  : Compute I*G_k = 1/2 (Z_k - Z*_{N-k})
		
			// Tmp =  {+,-}ComplexConjugate( Z_{N-k})
			float Tmp = SharedReal[NmK]; 
		
			Tmp *= (K > Non2)? -1 : 1;

			LocalBuffer[i].x += Tmp;
		}
	}

	if (Head == 0 ) LocalBuffer[0].x = 2.f * SharedReal[0];

	FFTMemoryBarrier();

	CopyLocalYToGroupShared(LocalBuffer, Head, Stride);

	FFTMemoryBarrier();

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			//
			// K = N/2 - abs(SrcIdx - N/2)

			// DstK = SrcIdx % Non2;
			// N - k
		
			uint NmK = (K > 0) ? ( N - K) : 0;

			// Z_k = LocalBuffer[i]
			// If k < N/2  : Store  F_k = 1/2 (Z_k + Z*_{N-k})
			// If k > N/2  : Compute I*G_k = 1/2 (Z_k - Z*_{N-k})
		
			// Tmp =  {+,-}ComplexConjugate( Z_{N-k})
			float Tmp = -SharedReal[NmK];
		
			Tmp *= (K < Non2)? 1 : -1;

			LocalBuffer[i].y += Tmp;
			//LocalBuffer[i] *= 0.5; 
		}
	}

	if (Head == 0) LocalBuffer[0].y = 2.f * SharedReal[0];
 
	{
		UNROLL for (uint i = 0; i < RADIX; ++i) LocalBuffer[i] *= 0.5;
	}

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			// If k > N/2 get G_k from I*G_k:  G_k = -I * (I G_k)
			if (K > Non2) LocalBuffer[i] = ComplexMult(Complex(0, -1), LocalBuffer[i] );
		}
	}
}

#endif  // SHARED_MEMORY_BUFFER_COUNT

void SplitTwoForOne(inout Complex LocalBuffer[2][RADIX], in uint Head, in uint Stride, in uint ArrayLength)
{
	
	 SplitTwoForOne(LocalBuffer[ 0 ], Head, Stride, ArrayLength);
	 FFTMemoryBarrier();
	 
	 SplitTwoForOne(LocalBuffer[ 1 ], Head, Stride, ArrayLength);
	
}

// Inverse of SplitTwoForOne()
// Combine the FFT of two real signals (f,g) into a single complex signal
// for use in inverting a "two-for-one" FFT .
//
//
// On return the transform of f and g have been separated and stored
// in the LocalBuffer in a form consistent with the packing the x-signal transform forward, 
// followed by the y-signal transform in reverse.
// 
// {F_o, G_o}, {F_1}, {F_2},..,{F_{N/2-1}}, {F_N/2, G_N/2}, {G_{N/2 +1}},  {G_{N/2 +2}}, .., { G_{N-2}},    {G_{N-1}} 
//     0       , 1    , 2    ,..., N/2-1,       N/2,           N/2 +1,      N/2 +2,      ..,  N/2- (2-N/2),  N/2 - (1-N/2)
// here {} := Complex
//
// On return the data in LocalBuffer across the threads in this group
// hold the transform  of the complex signal z = f + ig of lenght N.
//
#if SHARED_MEMORY_BUFFER_COUNT==2
void MergeTwoForOne(inout Complex LocalBuffer[RADIX], in uint Head, in uint Stride, in uint N)
{
	
	int Non2 = N / 2;

	// Write the two FFTs into shared memory.

	CopyLocalToGroupSharedWSync(LocalBuffer, Head, Stride);


	// Compose the transform of a single f+ig signal from the two real signals in the LocalBuffer
	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
	
			// N - k 
			uint NmK = (K > 0) ? (N - K) : 0 ;
	
			
		
			// If k < N/2  : LocalBuffer[i] = F_k,  
			//               Shared[N-K] =  G[N-K] = Conjugate(G[k]) =  (G[k]_r, -G[k]_i) = Complex(G_r, -G_i)
			//                          want   I G[k] =  float(-G[k]_i, G[k]_r)
			//                 Tmp = I * G_k   
			// If k > N/2  : LocalBuffer[i] = G_k 
			//               Shared[N-K] = F[N-k] = Conjugate(F[k]) = Complex(F[k]_r, -F[k]_i)
			//                           want -I F[k] = Complex(F[k]_i, -F[k]_r)
			//                 Tmp = -I * F   
			//  ComplexConjugate( Z_{N-k})

			// Tmp = (K < Non2) ? I *  G_k : -I * F
			Complex Tmp = Complex( SharedImag[ NmK ], SharedReal[ NmK ] );
			Tmp *= (K > Non2) ? -1 : 1;
		
			LocalBuffer[i] += Tmp;
		}
	}
	// Compose the transform of a single f+ig signal from the two real signals in the LocalBuffer

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			// if k > N/2 we have G - I *  F.  Multiply by I to get F + I * G
			if (K > Non2) LocalBuffer[ i ] = ComplexMult(Complex(0, 1), LocalBuffer[ i ]);

			if (K == Non2)
			{	
				// F_N/2 + I * G_N/2
				LocalBuffer[ i ] = Complex(SharedReal[ Non2 ], SharedImag[ Non2 ]);
			}

		}
	}

	if (Head == 0) 
	{
		// F_0 + I * G_0
		LocalBuffer[ 0 ] = Complex(SharedReal[ 0 ], SharedImag[ 0 ]);
 	}
}
#else  // SHARED_MEMORY_BUFFER_COUNT==1 
void MergeTwoForOne(inout Complex LocalBuffer[RADIX], in uint Head, in uint Stride, in uint N)
{
	
	uint Non2 = N / 2;

	// Write the two FFTs into shared memory.

	float TmpX[RADIX];
	{
		for (uint i = 0; i < RADIX; ++i) TmpX[i] = LocalBuffer[i].x;
	}

	CopyLocalYToGroupShared(LocalBuffer, Head, Stride);
	
	FFTMemoryBarrier();

	// Compose the transform of a single f+ig signal from the two real signals in the LocalBuffer
	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
	
			// N - k 
			uint NmK = (K > 0) ? (N - K) : 0 ;
	
			
		
			// If k < N/2  : LocalBuffer[i] = F_k,  
			//               Shared[N-K] =  G[N-K] = Conjugate(G[k]) =  (G[k]_r, -G[k]_i) = Complex(G_r, -G_i)
			//                          want   I G[k] =  float(-G[k]_i, G[k]_r)
			//                 Tmp = I * G_k   
			// If k > N/2  : LocalBuffer[i] = G_k 
			//               Shared[N-K] = F[N-k] = Conjugate(F[k]) = Complex(F[k]_r, -F[k]_i)
			//                           want -I F[k] = Complex(F[k]_i, -F[k]_r)
			//                 Tmp = -I * F   
			//  ComplexConjugate( Z_{N-k})

			// Tmp = (K < Non2) ? I *  G_k : -I * F
			float Tmp = SharedReal[ NmK ]; 
			Tmp *= (K > Non2) ? -1 : 1;
		
			LocalBuffer[i].x += Tmp;
		}
	}
	
	float2 FirstElement  = float2(0, SharedReal[0]);
	float2 MiddleElement = float2(0, SharedReal[Non2]);

	FFTMemoryBarrier();

	// Copy TmpX to GroupShared

	UNROLL for (uint r = 0, i = Head; r < RADIX; ++r, i += Stride)
	{
		SharedReal[ i ] = TmpX[ r ];
	}

	FFTMemoryBarrier();
	FirstElement.x  = SharedReal[0];
	MiddleElement.x = SharedReal[Non2];

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
	
			// N - k 
			uint NmK = (K > 0) ? (N - K) : 0 ;
	
			
		
			// If k < N/2  : LocalBuffer[i] = F_k,  
			//               Shared[N-K] =  G[N-K] = Conjugate(G[k]) =  (G[k]_r, -G[k]_i) = Complex(G_r, -G_i)
			//                          want   I G[k] =  float(-G[k]_i, G[k]_r)
			//                 Tmp = I * G_k   
			// If k > N/2  : LocalBuffer[i] = G_k 
			//               Shared[N-K] = F[N-k] = Conjugate(F[k]) = Complex(F[k]_r, -F[k]_i)
			//                           want -I F[k] = Complex(F[k]_i, -F[k]_r)
			//                 Tmp = -I * F   
			//  ComplexConjugate( Z_{N-k})

			// Tmp = (K < Non2) ? I *  G_k : -I * F
			float Tmp = SharedReal[ NmK ]; 
			Tmp *= (K > Non2) ? -1 : 1;
		
			LocalBuffer[i].y += Tmp;
		}
	}

	{
		UNROLL
		for (uint i = 0, K = Head; i < RADIX; ++i, K += Stride)
		{
			// if k > N/2 we have G - I *  F.  Multiply by I to get F + I * G
			if (K > Non2) LocalBuffer[ i ] = ComplexMult(Complex(0, 1), LocalBuffer[ i ]);

			if (K == Non2)
			{	
				// F_N/2 + I * G_N/2
				LocalBuffer[ i ] = MiddleElement;
			}
		}
	}

	if (Head == 0) LocalBuffer[ 0 ] = FirstElement;

}
#endif //SHARED_MEMORY_BUFFER_COUNT

void MergeTwoForOne(inout Complex LocalBuffer[2][RADIX], in uint Head, in uint Stride, in uint ArrayLength)
{
	
	 MergeTwoForOne(LocalBuffer[ 0 ], Head, Stride, ArrayLength);
	 FFTMemoryBarrier();
	 
	 MergeTwoForOne(LocalBuffer[ 1 ], Head, Stride, ArrayLength);
	 
}
