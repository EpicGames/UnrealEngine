// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "Hash.ush"

#if FEEDBACK_BUFFER_STRIDE == 1
#define FeedbackType uint
#elif FEEDBACK_BUFFER_STRIDE == 2
#define FeedbackType uint2
#endif

// Output compacted buffer
RWStructuredBuffer<uint2> RWCompactedFeedbackBuffer;
uint CompactedFeedbackBufferSize;
uint CompactedFeedbackCountShiftBits;

// Input feedback buffer and allocation size count
StructuredBuffer<uint> FeedbackBufferAllocator;
StructuredBuffer<FeedbackType> FeedbackBuffer;
uint FeedbackBufferSize;

// Indirect args
RWBuffer<uint> RWBuildHashTableIndirectArgs;

// Feedback element hashes
RWStructuredBuffer<uint> RWHashTableKeys;

// Deduplicated feedback elements
RWStructuredBuffer<uint> RWHashTableElementIndices;
StructuredBuffer<uint> HashTableElementIndices;
RWStructuredBuffer<uint> RWHashTableElementCounts;
StructuredBuffer<uint> HashTableElementCounts;
uint HashTableSize;
uint HashTableIndexWrapMask;

/**
 * Setups FBuildFeedbackHashTableCS arguments to run one lane per feedback element
 */
[numthreads(1, 1, 1)]
void BuildFeedbackHashTableIndirectArgsCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	if (DispatchThreadId.x == 0)
	{
		WriteDispatchIndirectArgs(RWBuildHashTableIndirectArgs, 0, DivideAndRoundUp64(min(FeedbackBufferAllocator[0], FeedbackBufferSize)), 1, 1);
	}
}

/** 
 * Hash key generation functions for each FeedbackType
 */
uint GetHashKey(uint PackedFeedback)
{
	return MurmurMix(PackedFeedback);
}

uint GetHashKey(uint2 PackedFeedback)
{
	return MurmurAdd(MurmurMix(PackedFeedback.x), PackedFeedback.y);
}

/** 
 * Copy input feedback item to output for each FeedbackType
 */
uint2 ExpandInputType(uint PackedFeedback)
{
	return uint2(PackedFeedback, 0);
}

uint2 ExpandInputType(uint2 PackedFeedback)
{
	return PackedFeedback;
}

/** 
 * Takes input Key, which is a hash and tries to add it to the linear probing hash table
 * Returns true if element was added for the first time
 * Returns Index which is index to the hash table bucket
 */
bool HashTableAdd(uint Key, inout uint Index)
{
	// Just skip add if search gets too long
	const uint MaxLinearProbingSteps = 32;

	LOOP
	for (uint LinearProbingStep = 0; LinearProbingStep < MaxLinearProbingSteps; ++LinearProbingStep)
	{
		Index = (Key + LinearProbingStep) & HashTableIndexWrapMask;

		uint StoredKey = RWHashTableKeys[Index];
		if (StoredKey != Key)
		{
			if (StoredKey != 0)
			{
				continue;
			}

			uint PrevKey;
			InterlockedCompareExchange(RWHashTableKeys[Index], 0, Key, PrevKey);

			if (PrevKey == 0)
			{
				RWHashTableKeys[Index] = Key;
				return true;
			}
			else if (PrevKey != Key)
			{
				continue;
			}
		}

		break;
	}

	return false;
}

/**
 * Takes a list of feedback elements and builds a hash table with element counts
 */
[numthreads(THREADGROUP_SIZE, 1, 1)]
void BuildFeedbackHashTableCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint ElementIndex = DispatchThreadId.x;

	if (ElementIndex < FeedbackBufferAllocator[0])
	{
		FeedbackType PackedFeedback = FeedbackBuffer[ElementIndex];

		uint Key = GetHashKey(PackedFeedback);

		uint HashTableIndex = 0;
		const bool bAdded = HashTableAdd(Key, HashTableIndex);
		if (bAdded)
		{
			RWHashTableElementIndices[HashTableIndex] = ElementIndex;
		}

		InterlockedAdd(RWHashTableElementCounts[HashTableIndex], 1);
	}
}

/**
 * Compacts feedback element hash table into a unique and tightly packed array of feedback elements with counts
 */
[numthreads(THREADGROUP_SIZE, 1, 1)]
void CompactFeedbackHashTableCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint HashTableIndex = DispatchThreadId.x;

	if (HashTableIndex < HashTableSize)
	{
		const uint ElementCount = HashTableElementCounts[HashTableIndex];

		if (ElementCount > 0)
		{
			uint WriteOffset = 0;
			InterlockedAdd(RWCompactedFeedbackBuffer[0].x, 1, WriteOffset);

			// Skip header element
			WriteOffset += 1;

			if (WriteOffset < CompactedFeedbackBufferSize)
			{
				uint ElementIndex = HashTableElementIndices[HashTableIndex];

				if (ElementIndex < FeedbackBufferSize)
				{
					uint2 PackedFeedback = ExpandInputType(FeedbackBuffer[ElementIndex]);

					// Store element count in compacted .y
					PackedFeedback.y |= (ElementCount << CompactedFeedbackCountShiftBits);

					RWCompactedFeedbackBuffer[WriteOffset] = PackedFeedback;
				}
			}
		}
	}
}
