// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "GPUMessaging.ush"

[numthreads(1, 1, 1)]
void ClearMessageBuffer()
{
	GPUMessageDataBuffer[0] = 0;
}