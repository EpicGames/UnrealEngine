// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Buffer containing both headers, message payloads and the global offset in element 0.
RWStructuredBuffer<uint> GPUMessageDataBuffer;
uint GPUMessageDataBufferSize;

struct FGPUMessageWriter
{
	bool bValid;
	uint PayloadOffset;
	uint PayloadNumUints;
	uint CurrentWriteOffset;
};

FGPUMessageWriter GPUMessageBegin(uint ID, uint PayloadNumUints)
{
	uint MessageStartOffset = 0U;
	InterlockedAdd(GPUMessageDataBuffer[0], 2U + PayloadNumUints, MessageStartOffset);

	// Always offset 1 for the GPUMessageDataBuffer[0] is used for the counter
	MessageStartOffset += 1U;

	FGPUMessageWriter GPUMessageWriter;
	GPUMessageWriter.PayloadOffset = MessageStartOffset + 2U;
	GPUMessageWriter.PayloadNumUints = PayloadNumUints;
	GPUMessageWriter.bValid = MessageStartOffset + 2U + PayloadNumUints <= GPUMessageDataBufferSize;
	GPUMessageWriter.CurrentWriteOffset = 0U;
	if (GPUMessageWriter.bValid)
	{
		GPUMessageDataBuffer[MessageStartOffset] = ID;
		GPUMessageDataBuffer[MessageStartOffset + 1] = PayloadNumUints;
	}
	return GPUMessageWriter;	
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, uint Value)
{
	if (Mw.bValid && Mw.CurrentWriteOffset < Mw.PayloadNumUints)
	{
		GPUMessageDataBuffer[Mw.PayloadOffset + Mw.CurrentWriteOffset] = Value;
		Mw.CurrentWriteOffset += 1U;
	}
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, uint2 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, uint3 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
	GPUMessageWriteItem(Mw, Value.z);
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, uint4 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
	GPUMessageWriteItem(Mw, Value.z);
	GPUMessageWriteItem(Mw, Value.w);
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, float Value)
{
	GPUMessageWriteItem(Mw, asuint(Value));
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, float2 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, float3 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
	GPUMessageWriteItem(Mw, Value.z);
}

void GPUMessageWriteItem(inout FGPUMessageWriter Mw, float4 Value)
{
	GPUMessageWriteItem(Mw, Value.x);
	GPUMessageWriteItem(Mw, Value.y);
	GPUMessageWriteItem(Mw, Value.z);
	GPUMessageWriteItem(Mw, Value.w);
}

void GPUMessagePost(uint ID, uint PayloadItem)
{
	FGPUMessageWriter Mw = GPUMessageBegin(ID, 1);
	GPUMessageWriteItem(Mw, PayloadItem);
}

void GPUMessagePost(uint ID, uint PayloadItem0, uint PayloadItem1)
{
	FGPUMessageWriter Mw = GPUMessageBegin(ID, 2);
	GPUMessageWriteItem(Mw, PayloadItem0);
	GPUMessageWriteItem(Mw, PayloadItem1);
}