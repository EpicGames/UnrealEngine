// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "GPUSceneWriter.ush"
#include "../InstanceCulling/InstanceCullingLoadBalancer.ush"

/**
 * Clear instances - set their primitive ID and set them to hidden.
 */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void GPUSceneSetInstancePrimitiveIdCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	FInstanceWorkSetup WorkSetup = InstanceCullingLoadBalancer_Setup(GroupId, GroupThreadIndex, 0U);

	if (!WorkSetup.bValid)
	{
		return;
	}

	const uint InstanceId = WorkSetup.Item.InstanceDataOffset + uint(WorkSetup.LocalItemIndex);
	const uint PrimitiveId = WorkSetup.Item.Payload;
	const uint InstanceFlags = INSTANCE_SCENE_DATA_FLAG_HIDDEN;
	WriteInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, InstanceFlags);
}
