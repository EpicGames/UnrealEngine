// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Include access API to update the GPU scene from a compute shader. 
 * This header must be included before SceneData.ush, as it modifies the behaviour. 
 * Expects FGPUSceneWriterParameters to be present in the parameter struct (use GPUScene::BeginReadWriteAccess/EndReadWriteAccess/GetWriteParameters)
 * Since this uses the GPU scene loading functions via a global RW buffer this cannot be used in the same translation unit as regular use of 
 * SceneData.ush (where the data is loaded from e.g., Scene.GPUScene.GPUSceneInstanceSceneData).
 */

#define USE_GPU_SCENE_DATA_RW 1

#include "../SceneData.ush"

void StoreInstanceSceneDataElement(uint InstanceId, uint ElementIndex, float4 DataElement)
{
	checkSlow(InstanceId < GetSceneData().MaxAllocatedInstanceId);
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	GPUSceneInstanceSceneDataRW[CalcInstanceDataIndex(InstanceId, ElementIndex)] = DataElement;
#else
	GPUSceneWriter.GPUSceneInstanceSceneDataRW[CalcInstanceDataIndex(InstanceId, ElementIndex)] = DataElement;
#endif
}

void StoreInstanceSceneDataElementX(uint InstanceId, uint ElementIndex, float Data)
{
	checkSlow(InstanceId < GetSceneData().MaxAllocatedInstanceId);
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	// NOTE: Structured buffers allow for DWORD granularity storage, so you don't have to write the whole float4
	GPUSceneInstanceSceneDataRW[CalcInstanceDataIndex(InstanceId, ElementIndex)].x = Data;
#else
	// NOTE: Structured buffers allow for DWORD granularity storage, so you don't have to write the whole float4
	GPUSceneWriter.GPUSceneInstanceSceneDataRW[CalcInstanceDataIndex(InstanceId, ElementIndex)].x = Data;
#endif
}

void StoreInstancePayloadDataElement(uint PayloadDataOffset, uint ElementIndex, float4 DataElement)
{
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	GPUSceneInstancePayloadDataRW[PayloadDataOffset + ElementIndex] = DataElement;
#else
	GPUSceneWriter.GPUSceneInstancePayloadDataRW[PayloadDataOffset + ElementIndex] = DataElement;
#endif
}

void StoreInstancePayloadDataElementZW(uint PayloadDataOffset, uint ElementIndex, float2 DataElement)
{
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	GPUSceneInstancePayloadDataRW[PayloadDataOffset + ElementIndex].zw = DataElement;
#else
	GPUSceneWriter.GPUSceneInstancePayloadDataRW[PayloadDataOffset + ElementIndex].zw = DataElement;
#endif
}

/**
 * Helpers to load and store the primitive ID and flags for the specified instance, which are packed together
 */

void WriteInstancePrimitiveIdAndFlags(uint InstanceId, uint PrimitiveId, uint InstanceFlags)
{
	const uint Packed0 = PackPrimitiveIdAndInstanceFlags(PrimitiveId, InstanceFlags);
	StoreInstanceSceneDataElementX(InstanceId, 0, asfloat(Packed0));
}

/**
 * Store the contents of InstanceData.PrimitiveId to the GPUScene buffers. 
 * NOTE: As the primitive ID is packed with the instance flags, the flags are loaded and re-stored
 */
void WriteInstancePrimitiveId(uint InstanceId, uint NewPrimitiveId)
{
	uint OldPrimitiveId, InstanceFlags;
	LoadInstancePrimitiveIdAndFlags(InstanceId, OldPrimitiveId, InstanceFlags);
	WriteInstancePrimitiveIdAndFlags(InstanceId, NewPrimitiveId, InstanceFlags);
}

/**
 * Store the contents of InstanceData.LocalToWorld to the GPUScene buffers. Optionally, also calculates the determinant
 * of the transform's rotation to set or unset the determinant sign flag on the instance flags.
 *
 * NOTE: With Large World Coordinates, the transform provided is expected to be relative to the primitive's PositionHigh.
 * Use the WriteInstanceLocalToWorld helper if you don't already have a relative transform.
 **/
void WriteInstanceLocalToRelativeWorld(uint InstanceId, float4x4 LocalToRelativeWorld, bool StoreDeterminantSign)
{
#if PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS
	uint4 RotationScale;
	float3 Translation;
	EncodeTransform(LocalToRelativeWorld, RotationScale, Translation);
	StoreInstanceSceneDataElement(InstanceId, 1, asfloat(RotationScale));
	StoreInstanceSceneDataElement(InstanceId, 2, float4(Translation, 0.0f));
#else
	LocalToRelativeWorld = transpose(LocalToRelativeWorld);
	StoreInstanceSceneDataElement(InstanceId, 1, LocalToRelativeWorld[0]);
	StoreInstanceSceneDataElement(InstanceId, 2, LocalToRelativeWorld[1]);
	StoreInstanceSceneDataElement(InstanceId, 3, LocalToRelativeWorld[2]);
#endif

	// check to update the determinant sign flag on the instance flags as well
	if (StoreDeterminantSign)
	{
		// Check to update the determinant sign flag on the instance flags
		uint PrimitiveId, InstanceFlags;
		LoadInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, InstanceFlags);
		SetInstanceDeterminantSignFlag(determinant((float3x3)LocalToRelativeWorld), InstanceFlags);
		WriteInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, InstanceFlags);
	}
}

void WriteInstanceLocalToWorld(uint InstanceId, uint PrimitiveId, FDFMatrix LocalToWorld, bool StoreDeterminantSign)
{
	// Ensure the LocalToWorld matrix is relative to the primitive tile position
	float4x4 LocalToRelativeWorld = DFMultiplyAffineTranslationDemote(LocalToWorld, -GetPrimitiveData(PrimitiveId).PositionHigh);
	WriteInstanceLocalToRelativeWorld(InstanceId, LocalToRelativeWorld, StoreDeterminantSign);
}

/**
 * Initializes the instance scene data of the specified instance data slot using a LocalToWorld matrix that is already relative to
 * its primitive's render tile position (When LWC. Otherwise, the matrix is simply LocalToWorld). Use InitializeInstanceSceneDataWS
 * or InitializeInstanceSceneDataLS if you don't already have a LocalToRelativeWorldMatrix
 * NOTE: This function must match FInstanceSceneShaderData::BuildInternal
 **/
 void InitializeInstanceSceneData(
 	uint InstanceId,
 	uint PrimitiveId,
 	uint InstanceRelativeId,
 	uint InstanceFlags, // combination of INSTANCE_SCENE_DATA_FLAG_
 	uint CustomDataCount,
 	float RandomId,
 	float4x4 LocalToRelativeWorld)
 {
 	SetInstanceDeterminantSignFlag(determinant((float3x3)LocalToRelativeWorld), InstanceFlags);
 	
 	float4 Element0 =
 	{
 		asfloat(PackPrimitiveIdAndInstanceFlags(PrimitiveId, InstanceFlags)),
 		asfloat(PackInstanceRelativeIdAndCustomDataCount(InstanceRelativeId, CustomDataCount)),
 		asfloat(GetSceneData().FrameNumber),
 		RandomId
	};
 	StoreInstanceSceneDataElement(InstanceId, 0, Element0);
 	WriteInstanceLocalToRelativeWorld(InstanceId, LocalToRelativeWorld, false);
 }

/** This version initializes the instance with a world-space transform */
void InitializeInstanceSceneDataWS(
 	uint InstanceId,
 	uint PrimitiveId,
 	uint InstanceRelativeId,
 	uint PayloadDataFlags,
 	uint CustomDataCount,
 	float RandomId,
 	FDFMatrix LocalToWorld)
 {
	 float4x4 LocalToRelativeWorld = DFMultiplyAffineTranslationDemote(LocalToWorld, -GetPrimitiveData(PrimitiveId).PositionHigh);
	 InitializeInstanceSceneData(InstanceId, PrimitiveId, InstanceRelativeId, PayloadDataFlags, CustomDataCount, RandomId, LocalToRelativeWorld);
 }

/** This version initializes the instance with a transform that is relative to its primitive's local space */
void InitializeInstanceSceneDataLS(
 	uint InstanceId,
 	uint PrimitiveId,
 	uint InstanceRelativeId,
 	uint PayloadDataFlags,
 	uint CustomDataCount,
 	float RandomId,
 	float4x4 LocalToPrimitive)
{
	// Multiply the LS by the primitive's LocalToWorld, then the internal matrix is guaranteed to be LocalToRelativeWorld
	// because PositionHigh is copied from the primitive LocalToWorld
	
	float4x4 LocalToWorld = DFMultiply(LocalToPrimitive, GetPrimitiveData(PrimitiveId).LocalToWorld).M;
	InitializeInstanceSceneData(InstanceId, PrimitiveId, InstanceRelativeId, PayloadDataFlags, CustomDataCount, RandomId, LocalToWorld);
}

/**
 * Helper for loading instance payload offsets for GPU writers (assumes global parameter for SOA stride)
 **/
 FInstancePayloadDataOffsets GetInstancePayloadDataOffsets(uint InstanceId)
{
	uint PrimitiveId, InstanceFlags, InstanceRelativeId, CustomDataCount;
	LoadInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, InstanceFlags);
	LoadInstanceRelativeIdAndCustomDataCount(InstanceId, InstanceRelativeId, CustomDataCount);
	return GetInstancePayloadDataOffsets(PrimitiveId, InstanceFlags, InstanceRelativeId);
}

/**
 * Store the local instance bounds to the GPU Scene sidecar payload buffer.
 **/
void WriteInstanceLocalBounds(FInstancePayloadDataOffsets PayloadOffsets, float3 LocalCenter, float3 LocalExtent)
{
	BRANCH
	if (PayloadOffsets.LocalBounds == INVALID_INSTANCE_PAYLOAD_OFFSET)
	{
		return;
	}

	StoreInstancePayloadDataElementZW(PayloadOffsets.LocalBounds, 0, LocalCenter.xy);
	StoreInstancePayloadDataElement(PayloadOffsets.LocalBounds, 1, float4(LocalCenter.z, LocalExtent.x, LocalExtent.y, LocalExtent.z));
}

void WriteInstanceLocalBounds(uint InstanceId, float3 LocalCenter, float3 LocalExtent)
{
	FInstancePayloadDataOffsets PayloadOffsets = GetInstancePayloadDataOffsets(InstanceId);
	WriteInstanceLocalBounds(PayloadOffsets, LocalCenter, LocalExtent);
}

/**
 * Store the data for dynamic instances to the GPUScene sidecar payload buffer. Use WriteInstanceDynamicDataWS or 
 * WriteInstanceDynamicDataLS if you don't have a relative transform.
 **/
void WriteInstanceDynamicData(FInstancePayloadDataOffsets PayloadOffsets, float4x4 PrevLocalToRelativeWorld)
{	
	BRANCH
	if (PayloadOffsets.DynamicData == INVALID_INSTANCE_PAYLOAD_OFFSET)
	{
		// Instance doesn't have dynamic data
		return;
	}

#if PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS
	uint4 RotationScale;
	float3 Translation;
	EncodeTransform(PrevLocalToRelativeWorld, RotationScale, Translation);
	StoreInstancePayloadDataElement(PayloadOffsets.DynamicData, 0, asfloat(RotationScale));
	StoreInstancePayloadDataElement(PayloadOffsets.DynamicData, 1, float4(Translation, 0.0f));
#else
	PrevLocalToRelativeWorld = transpose(PrevLocalToRelativeWorld);
	StoreInstancePayloadDataElement(PayloadOffsets.DynamicData, 0, PrevLocalToRelativeWorld[0]);
	StoreInstancePayloadDataElement(PayloadOffsets.DynamicData, 1, PrevLocalToRelativeWorld[1]);
	StoreInstancePayloadDataElement(PayloadOffsets.DynamicData, 2, PrevLocalToRelativeWorld[2]);
#endif
}

void WriteInstanceDynamicData(uint InstanceId, float4x4 PrevLocalToRelativeWorld)
{
	WriteInstanceDynamicData(GetInstancePayloadDataOffsets(InstanceId), PrevLocalToRelativeWorld);
}

/** This version computes the relative transform from the specified world transform */
void WriteInstanceDynamicDataWS(FInstancePayloadDataOffsets PayloadOffsets, uint PrimitiveId, FDFMatrix PrevLocalToWorld)
{
	float4x4 PrevLocalToRelativeWorld = DFMultiplyAffineTranslationDemote(PrevLocalToWorld, -GetPrimitiveData(PrimitiveId).PositionHigh);
	WriteInstanceDynamicData(PayloadOffsets, PrevLocalToRelativeWorld);
}

void WriteInstanceDynamicDataWS(uint InstanceId, uint PrimitiveId, FDFMatrix PrevLocalToWorld)
{
	WriteInstanceDynamicDataWS(GetInstancePayloadDataOffsets(InstanceId), PrimitiveId, PrevLocalToWorld);
}

/** This version computes the relative transform from the specified local-space transform */
void WriteInstanceDynamicDataLS(FInstancePayloadDataOffsets PayloadOffsets, uint PrimitiveId, float4x4 PrevLocalToPrimitive)
{
	FDFMatrix PrevLocalToWorld = DFMultiply(PrevLocalToPrimitive, GetPrimitiveData(PrimitiveId).PreviousLocalToWorld);
	WriteInstanceDynamicDataWS(PayloadOffsets, PrimitiveId, PrevLocalToWorld);
}

void WriteInstanceDynamicDataLS(uint InstanceId, uint PrimitiveId, float4x4 PrevLocalToPrimitive)
{
	WriteInstanceDynamicDataLS(GetInstancePayloadDataOffsets(InstanceId), PrimitiveId, PrevLocalToPrimitive);
}

/**
 * Stores a float4 element of custom data for the given instance
 */
 void WriteInstanceCustomData(FInstancePayloadDataOffsets PayloadOffsets, uint ElementIndex, float4 ElementData)
 {
 	BRANCH
 	if (PayloadOffsets.CustomData == INVALID_INSTANCE_PAYLOAD_OFFSET)
 	{
		// Instance has no custom data
 		return;
 	}

 	StoreInstancePayloadDataElement(PayloadOffsets.CustomData, ElementIndex, ElementData);
 }

 void WriteInstanceCustomData(uint InstanceId, uint ElementIndex, float4 ElementData)
 {
	FInstancePayloadDataOffsets PayloadOffsets = GetInstancePayloadDataOffsets(InstanceId);
 	WriteInstanceCustomData(PayloadOffsets, ElementIndex, ElementData);
 }