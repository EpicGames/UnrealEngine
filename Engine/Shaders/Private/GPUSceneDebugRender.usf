// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SceneData.ush"
#include "ShaderPrint.ush"

int bDrawAll;
int bDrawUpdatedOnly;

int2 InputCoord;
float3 PickingRayStart;
float3 PickingRayEnd;
float DrawRange;

//////////////////////////////////////////////////////////////////////////////////////////
#define INVALID_PRIMITIVEID ~0

uint					SelectedNameInfoCount;
uint					SelectedNameCharacterCount;
StructuredBuffer<uint2> SelectedPrimitiveNameInfos;
Buffer<uint>			SelectedPrimitiveNames;

struct FPrimitiveSceneDebugNameInfo
{
	uint PrimitiveID;
	uint Length;
	uint Offset;
	uint Pad0;
	uint Pad1;
};

FPrimitiveSceneDebugNameInfo UnpackDebugNameInfo(uint2 In)
{
	FPrimitiveSceneDebugNameInfo Out = (FPrimitiveSceneDebugNameInfo)0;
	Out.PrimitiveID = In.x;
	Out.Offset = (In.y) & 0xFFFF;
	Out.Length = (In.y>>16) & 0xFF;

	return Out;
}

FPrimitiveSceneDebugNameInfo FindNameInfo(uint PrimitiveID)
{
	FPrimitiveSceneDebugNameInfo Out = (FPrimitiveSceneDebugNameInfo)0;
	Out.PrimitiveID = INVALID_PRIMITIVEID;
	for (uint It = 0; It < SelectedNameInfoCount; ++It)
	{
		if (SelectedPrimitiveNameInfos[It].x == PrimitiveID)
		{
			return UnpackDebugNameInfo(SelectedPrimitiveNameInfos[It]);
		}
	}
	return Out;
}

void PrintInstanceName(inout FShaderPrintContext Context, uint PrimitiveID, FFontColor InColor)
{
	const FPrimitiveSceneDebugNameInfo Info = FindNameInfo(PrimitiveID);
	if (Info.PrimitiveID != INVALID_PRIMITIVEID && (Info.Length + Info.Offset) <= SelectedNameCharacterCount)
	{
		for (uint It = 0; It < Info.Length; ++It)
		{
			const uint Char = SelectedPrimitiveNames[It + Info.Offset];
			PrintSymbol(Context, Char, InColor);
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////

StructuredBuffer<uint> SelectedPrimitiveFlags;
RWBuffer<uint> RWDrawCounter;

void AddFlag(inout FShaderPrintContext Context, bool bEnabled)
{
	if (bEnabled)
	{
		Print(Context, TEXT("On     "), FontGreen);
	}
	else
	{
		Print(Context, TEXT("Off    "), FontRed);
	}
}
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void GPUSceneDebugRenderCS(uint InstanceID : SV_DispatchThreadID)
{
	if (InstanceID >= GetSceneData().MaxAllocatedInstanceId)
	{
		return;
	}

	FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceID);

	// Early out on invalid ID
	if (InstanceData.PrimitiveId == 0xFFFFFFFFu)
	{
		return;
	}

	// Only show updated
	if (bDrawUpdatedOnly && InstanceData.LastUpdateSceneFrameNumber != GetSceneData().FrameNumber)
	{
		return;
	}

	float3 LocalRayStart = mul(float4(PickingRayStart, 1.0f), DFHackToFloat(InstanceData.WorldToLocal)).xyz;

	// Skip if beyond draw range
	if (DrawRange > 0.0f && length(InstanceData.LocalBoundsCenter - LocalRayStart) > DrawRange)
	{
		return;
	}

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

	const float3 InstanceBoxBoundsCenter = InstanceData.LocalBoundsCenter;
	const float3 InstanceBoxBoundsExtent = InstanceData.LocalBoundsExtent;

	float3 LocalRayEnd = mul(float4(PickingRayEnd, 1.0f), DFHackToFloat(InstanceData.WorldToLocal)).xyz;
	float2 RayResult = LineBoxIntersect(LocalRayStart, LocalRayEnd, InstanceBoxBoundsCenter - InstanceBoxBoundsExtent, InstanceBoxBoundsCenter + InstanceBoxBoundsExtent);
	bool bIsMouseOver = RayResult.x <= RayResult.y;
	// Only draw selected
	const uint BitsPerWord = 32U;
	const uint Mask = 1U << (InstanceData.PrimitiveId % BitsPerWord);

	bool bIsSelected = (InstanceData.PrimitiveId < GetSceneData().MaxPersistentPrimitiveIndex && (SelectedPrimitiveFlags[InstanceData.PrimitiveId / BitsPerWord] & Mask) != 0U);

	//float4 InstanceBoundColor = InstanceData.PrimitiveId < GetSceneData().MaxPersistentPrimitiveIndex ? float4(0.3f, 1.0f, 0.5f, 1.0f) : float4(0.3f, 0.3f, 0.3f, 0.5f);
	float4 InstanceBoundColor = float4(ColorSilver.xyz, 0.5f);

	if (bDrawAll)
	{
		InstanceBoundColor = ColorLightGreen;
	}
	
	if (bIsMouseOver)
	{
		InstanceBoundColor = ColorYellow;
	}
	
	if (bIsSelected)
	{
		InstanceBoundColor = ColorOrange;
	}

	if (bDrawUpdatedOnly)
	{
		InstanceBoundColor = ColorRed;
	}
	if (bIsMouseOver || bIsSelected || bDrawAll != 0)
	{
		AddOBBWS(InstanceData.LocalBoundsCenter - InstanceData.LocalBoundsExtent, InstanceData.LocalBoundsCenter + InstanceData.LocalBoundsExtent, InstanceBoundColor, DFHackToFloat(InstanceData.LocalToWorld));
		AddReferentialWS(DFHackToFloat(InstanceData.LocalToWorld), 50.f);
	}

	uint StartOffsetX = 10;
	uint StartOffsetY = 30;
	uint SlotHeight = 8;
	if (InstanceID == 0 && SelectedNameInfoCount>0)
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(StartOffsetX, StartOffsetY + SlotHeight));
		Print(Context, TEXT("Prim.ID     Inst.ID     Shadow Veloc. CusDat DynDat Name"), FontWhite);
	}

	if (bIsSelected)
	{	
		uint SlotIndex = 0;
		InterlockedAdd(RWDrawCounter[0], 1, SlotIndex);

		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(StartOffsetX, StartOffsetY + 2 * SlotHeight + SlotIndex * SlotHeight));

		Print(Context, InstanceData.PrimitiveId, FontYellow);
		Print(Context, InstanceID, FontOrange);

		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);
		AddFlag(Context, 0u != (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS));
		AddFlag(Context, 0u != (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_OUTPUT_VELOCITY));
		AddFlag(Context, 0u != (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA));
		AddFlag(Context, 0u != (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA));
		
		PrintInstanceName(Context, InstanceData.PrimitiveId, FontSilver); Newline(Context);
	}
}
