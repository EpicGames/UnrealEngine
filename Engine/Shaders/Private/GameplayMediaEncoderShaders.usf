// Copyright Epic Games, Inc. All Rights Reserved.

/*=========================================================================================
	GameplayMediaEncoderShaders.usf: utility shaders for the GameplayMediaEncoder module
=========================================================================================*/

#include "Common.ush"

Texture2D InTexture;
SamplerState InTextureSampler;

// Swizzle color channels from RGBA to BGRA
void ScreenSwizzlePS(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
    OutColor = Texture2DSample(InTexture, InTextureSampler, Input.UV).bgra;
}
