// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

half3 LinearTo709Branchless(half3 lin) 
{
	return min(lin * 4.5, pow(max(lin, 0.018), 0.45) * 1.099 - 0.099);
}

half3 Rec709ToLinear(half3 Color)
{
	// abs() added to silence compiler warning about computing pow() of negative number (both sides of the select are evaluated).
	return select(Color > 0.081, pow((abs(Color) + 0.099) / 1.099, 1.0 / 0.45), Color / 4.5);
}

half3 LinearToSrgbBranchless(half3 lin) 
{
	return min(lin * 12.92, pow(max(lin, 0.00313067), 1.0/2.4) * 1.055 - 0.055);
	// Possible that mobile GPUs might have native pow() function?
	//return min(lin * 12.92, exp2(log2(max(lin, 0.00313067)) * (1.0/2.4) + log2(1.055)) - 0.055);
}

half LinearToSrgbBranchingChannel(half lin) 
{
	if(lin < 0.00313067) return lin * 12.92;
	return pow(lin, (1.0/2.4)) * 1.055 - 0.055;
}

half3 LinearToSrgbBranching(half3 lin) 
{
	return half3(
		LinearToSrgbBranchingChannel(lin.r),
		LinearToSrgbBranchingChannel(lin.g),
		LinearToSrgbBranchingChannel(lin.b));
}

#ifndef PLATFORM_PREFER_SRGB_BRANCH
#if FEATURE_LEVEL > FEATURE_LEVEL_ES3_1
#define PLATFORM_PREFER_SRGB_BRANCH 1
#else
#define PLATFORM_PREFER_SRGB_BRANCH 0
#endif
#endif

half3 LinearToSrgb(half3 lin) 
{
#if PLATFORM_PREFER_SRGB_BRANCH
	// Branching is faster than branchless on AMD on PC.
	return LinearToSrgbBranching(lin);
#else
	// Adreno devices(Nexus5) with Android 4.4.2 do not handle branching version well, so always use branchless on Mobile
	return LinearToSrgbBranchless(lin);
#endif
}

half3 sRGBToLinear( half3 Color ) 
{
	// abs() added to silence compiler warning about computing pow() of negative number (both sides of the select are evaluated).
	return select(Color > 0.04045, pow( abs(Color) * (1.0 / 1.055) + 0.0521327, 2.4 ), Color * (1.0 / 12.92));
}

/**
 * @param GammaCurveRatio The curve ratio compared to a 2.2 standard gamma, e.g. 2.2 / DisplayGamma.  So normally the value is 1.
 */
half3 ApplyGammaCorrection(half3 LinearColor, half GammaCurveRatio)
{
	// Apply "gamma" curve adjustment.
	half3 CorrectedColor = pow(LinearColor, GammaCurveRatio);

	#if MAC
		// Note, MacOSX native output is raw gamma 2.2 not sRGB!
		CorrectedColor = pow(CorrectedColor, 1.0/2.2);
	#else
		#if USE_709
			// Didn't profile yet if the branching version would be faster (different linear segment).
			CorrectedColor = LinearTo709Branchless(CorrectedColor);
		#else
			CorrectedColor = LinearToSrgb(CorrectedColor);
		#endif
	#endif

	return CorrectedColor;
}

//
// Generic log lin transforms
//
float3 LogToLin( float3 LogColor )
{
	const float LinearRange = 14;
	const float LinearGrey = 0.18;
	const float ExposureGrey = 444;

	// Using stripped down, 'pure log', formula. Parameterized by grey points and dynamic range covered.
	float3 LinearColor = exp2( ( LogColor - ExposureGrey / 1023.0 ) * LinearRange ) * LinearGrey;
	//float3 LinearColor = 2 * ( pow(10.0, ((LogColor - 0.616596 - 0.03) / 0.432699)) - 0.037584 );	// SLog
	//float3 LinearColor = ( pow( 10, ( 1023 * LogColor - 685 ) / 300) - .0108 ) / (1 - .0108);	// Cineon
	//LinearColor = max( 0, LinearColor );

	return LinearColor;
}

float3 LinToLog( float3 LinearColor )
{
	const float LinearRange = 14;
	const float LinearGrey = 0.18;
	const float ExposureGrey = 444;

	// Using stripped down, 'pure log', formula. Parameterized by grey points and dynamic range covered.
	float3 LogColor = log2(LinearColor) / LinearRange - log2(LinearGrey) / LinearRange + ExposureGrey / 1023.0;	// scalar: 3log2 3mad
	//float3 LogColor = (log2(LinearColor) - log2(LinearGrey)) / LinearRange + ExposureGrey / 1023.0;
	//float3 LogColor = log2( LinearColor / LinearGrey ) / LinearRange + ExposureGrey / 1023.0;
	//float3 LogColor = (0.432699 * log10(0.5 * LinearColor + 0.037584) + 0.616596) + 0.03;	// SLog
	//float3 LogColor = ( 300 * log10( LinearColor * (1 - .0108) + .0108 ) + 685 ) / 1023;	// Cineon
	LogColor = saturate( LogColor );

	return LogColor;
}

//
// Pseudo-ACES 100 nit transforms
//
float
aces100nitFitInverseFloat(float x) 
{
	x = max(0.f, min(0.99f, x));
	//float c = ((-6.32456e-8*pow((-21510484096000.0*x*x + 26714646293200.0*x + 27735750507.0), 0.5) -0.146704*x + 0.0083284)) * rcp( x - 1.01654);
	float c = ( -0.632456 * sqrt( -0.21510484096 *x*x + 0.267146462932 * x + 0.00027735750507 ) - 0.146704 * x + 0.0083284 ) / ( x - 1.01654 );

	// Clamp to half float range
	return max(0.f, min(65504.f, c)); 	
}

float3
aces100nitFitInverse(float3 FilmColor) 
{
	float3 inverse;
	inverse.r = aces100nitFitInverseFloat(FilmColor.r);
	inverse.g = aces100nitFitInverseFloat(FilmColor.g);
	inverse.b = aces100nitFitInverseFloat(FilmColor.b);
	return inverse;
}

static const float UE_SCRGB_WHITE_NITS = 80.; // ScRGB D65 white luminance (cd/m^2)
static const float UE_PQ_WHITE_NITS = 10000.;

//
// Dolby PQ transforms
//
float3
ST2084ToLinearNormalized(float3 pq)
{
	const float m1 = 0.1593017578125; // = 2610. / 4096. * .25;
	const float m2 = 78.84375; // = 2523. / 4096. *  128;
	const float c1 = 0.8359375; // = 2392. / 4096. * 32 - 2413./4096.*32 + 1;
	const float c2 = 18.8515625; // = 2413. / 4096. * 32;
	const float c3 = 18.6875; // = 2392. / 4096. * 32;

	float3 Np = pow( pq, 1./m2 );
	float3 L = Np - c1;
	L = max(0., L);
	L = L / (c2 - c3 * Np);
	L = pow( L, 1./m1 );
	float3 P = L;

	return P;
}

float3
ST2084ToLinear(float3 pq)
{
	return ST2084ToLinearNormalized(pq) * UE_PQ_WHITE_NITS;
}

float3
LinearToST2084Normalized(float3 lin)
{
	const float m1 = 0.1593017578125; // = 2610. / 4096. * .25;
	const float m2 = 78.84375; // = 2523. / 4096. *  128;
	const float c1 = 0.8359375; // = 2392. / 4096. * 32 - 2413./4096.*32 + 1;
	const float c2 = 18.8515625; // = 2413. / 4096. * 32;
	const float c3 = 18.6875; // = 2392. / 4096. * 32;

	float3 L = lin;
	float3 Lm = pow(L, m1);
	float3 N1 = ( c1 + c2 * Lm );
	float3 N2 = ( 1.0 + c3 * Lm );
	float3 N = N1 * rcp(N2);
	float3 P = pow( N, m2 );
	
	return P;
}

float3
LinearToST2084(float3 lin)
{
	return LinearToST2084Normalized(lin / UE_PQ_WHITE_NITS);
}

//
// Sony S-Log3
//
float3
SLog3ToLinear(float3 Value)
{
	return select((Value >= 171.2102946929f / 1023.0f), (pow(10.0f, (Value * 1023.0f - 420.f) / 261.5f)) * 0.19f - 0.01f, (Value * 1023.0f - 95.0f) * 0.01125000f / (171.2102946929f - 95.0f));
}

float LinearToHLG(float Value)
{
	const float HLG_a = 0.17883277f;
	const float HLG_b = 0.28466892f; // 1.0 - 4.0 * HLG_a
	const float HLG_c = 0.559910729529562f; // 0.5 - HLG_a * log(4.0 * HLG_a)

	const float E = abs(Value);
	float Out = 0.0f;
	
	if (E <= 1.0f / 12.0f)
	{
		Out = sqrt(3.0f * E);
	}
	else
	{
		Out = HLG_a * log(12.0f * E - HLG_b) + HLG_c;
	}

	return Out * sign(Value);
}

float HLGToLinear(float Value)
{
	const float HLG_a = 0.17883277f;
	const float HLG_b = 0.28466892f; // 1.0 - 4.0 * HLG_a
	const float HLG_c = 0.559910729529562f; // 0.5 - HLG_a * log(4.0 * HLG_a)

	const float Eprime = abs(Value);
	float Out = 0.0f;
	
	if (Eprime <= 0.5f)
	{
		Out = Eprime * Eprime / 3.0f;
	}
	else
	{
		Out = exp((Eprime - HLG_c) / HLG_a) + HLG_b;
		Out /= 12.0f;
	}

	return Out * sign(Value);
}

float3 LinearToHLG(float3 L)
{
	return float3(
		LinearToHLG(L.x),
		LinearToHLG(L.y),
		LinearToHLG(L.z)
	);
}

float3 HLGToLinear(float3 E)
{
	return float3(
		HLGToLinear(E.x),
		HLGToLinear(E.y),
		HLGToLinear(E.z)
	);
}

// Full BT.2100 HLG EOTF: inverse OETF followed by OOTF.
// OOTF: FD = alpha * Ys^(gamma-1) * E, with alpha=PeakLuminance, gamma=SystemGamma.
// Defaults to reference display: 1000 cd/m2, gamma=1.2.
float3 HLGToLinearWithOOTF(float3 Color, float PeakLuminance = 1000.0f, float SystemGamma = 1.2f)
{
	// BT.2020 luminance weights (ITU-R BT.2100-3 Table 5).
	const float3 Rec2020LuminanceWeights = float3(0.2627f, 0.6780f, 0.0593f);

	Color = HLGToLinear(Color);
	float Ys = dot(Color, Rec2020LuminanceWeights);

	return select(Ys > 0.0f, PeakLuminance * pow(Ys, SystemGamma - 1.0f) * Color, float3(0.0f, 0.0f, 0.0f));
}

