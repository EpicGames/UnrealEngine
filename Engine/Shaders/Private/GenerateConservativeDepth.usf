// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenerateConservativeDepth.usf
=============================================================================*/

#include "Common.ush"
#include "Definitions.usf"

#include "Nanite/NaniteHZBCull.ush"



RWTexture2D<float> ConservativeDepthTextureUAV;

int2 ConservativeDepthTextureSize;
int DestinationPixelSizeAtFullRes;

// TODO: Merge this with the similar implementations in LightGridInjection.usf and MegaLightsVolume.ush
FScreenRect ComputeFroxelCullRect(uint3 GridCoordinate, float FootprintMargin)
{
	// Compute extent of tiles in clip-space. Note that the last tile may extend a bit outside of view if view size is not evenly divisible tile size.
	const float2 InvCulledGridSizeF = DestinationPixelSizeAtFullRes * ResolvedView.ViewSizeAndInvSize.zw;
	const float2 TileSize = float2(2.0f, -2.0f) * InvCulledGridSizeF.xy;
	const float2 UnitPlaneMin = float2(-1.0f, 1.0f);

	float2 UnitPlaneTileMin = (GridCoordinate.xy - FootprintMargin) * TileSize + UnitPlaneMin;
	float2 UnitPlaneTileMax = (GridCoordinate.xy + 1 + FootprintMargin) * TileSize + UnitPlaneMin;

	float3 CullRectMin;
	CullRectMin.x = min(UnitPlaneTileMin.x, UnitPlaneTileMax.x);
	CullRectMin.y = min(UnitPlaneTileMin.y, UnitPlaneTileMax.y);
	CullRectMin.z = 0;

	float3 CullRectMax;
	CullRectMax.x = max(UnitPlaneTileMin.x, UnitPlaneTileMax.x);
	CullRectMax.y = max(UnitPlaneTileMin.y, UnitPlaneTileMax.y);
	CullRectMax.z = 0;

	return GetScreenRect(int4(0, 0, HZBViewSize), CullRectMin, CullRectMax, 4);
}

[numthreads(8, 8, 1)]
void GenerateConservativeDepthBufferCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	if (all(DispatchThreadId.xy < uint2(ConservativeDepthTextureSize)))
	{
		FScreenRect ScreenRect = ComputeFroxelCullRect(DispatchThreadId, 0.5f);

		ConservativeDepthTextureUAV[DispatchThreadId.xy] = GetMinDepthFromHZB(ScreenRect, true);
	}
}


