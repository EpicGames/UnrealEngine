// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Define types used by Generated Uniform buffers (which is pulled in by Common.ush)

// Nanite
#include "Nanite/NanitePackedNaniteView.ush"

// Hair Strands
#include "HairStrands/HairStrandsVisibilityCommonStruct.ush"
#include "HairStrands/HairStrandsVoxelPageCommonStruct.ush"
#include "HairStrands/HairStrandsDeepShadowCommonStruct.ush"

#include "/Engine/Shared/LightSceneData.h"
#include "/Engine/Shared/LightViewData.h"
#include "/Engine/Shared/RayTracingTypes.h"
#include "/Engine/Shared/NaniteDefinitions.h"