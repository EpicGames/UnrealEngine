// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCacheVertexFactory.hlsl: Vertex factory for geometry caches (supports motion blur data)
=============================================================================*/


#include "VertexFactoryCommon.ush"

/*
    Meshes vertices can optionally still be packed and will be unpacked using the following formula:
    VertextBuffer.Position*MeshExtension+MeshOrigin
*/
float3 MeshOrigin;
float3 MeshExtension;

/* This vertex type is passed in an extra float4 MotionBlurData. This allows specifying motion blur data in a generic way. The previous
    frame's object space position is derived from the MotionBlurData and the current unpacked object space position fields in the following way:

    (Vertex.MotionBlurData * MotionBlurDataExtension + MotionBlurDataOrigin) + (UnpackePosition * MotionBlurPositionScale)
    
    Using this formula a number of effects can be achieved with a single vertex shader. Some common examples
    - Packed previous vertex positions are bound to MotionBlurData
        - MotionBlurDataOrigin: Pevious frame's packed origin
        - MotionBlurDataExtension: Previous frame's packed scale multiplied with a time factor (to make motion blur fps independent)
        - MotionBlurPositionScale: 0
    - Explicit float 3 motion vectors are bound
        - MotionBlurDataOrigin: 0
        - MotionBlurDataExtension: A negative time factor (to make motion blur fps independent and move the velocities back in time)
        - MotionBlurPositionScale: 1
    - No motion blur data is not available for this frame (an undefined buffer is bound)
        - MotionBlurDataOrigin: 0
        - MotionBlurDataExtension: 0
        - MotionBlurPositionScale: 1
*/
float3 MotionBlurDataOrigin;
float3 MotionBlurDataExtension;
float MotionBlurPositionScale;

struct FVertexFactoryInput
{
	float4	Position		: ATTRIBUTE0;
	// 0..1
	HALF3_TYPE	TangentX		: ATTRIBUTE1;
	// 0..1
	// TangentZ.w contains sign of tangent basis determinant
	HALF4_TYPE	TangentZ		: ATTRIBUTE2;	

	// Per vertex color 
	float4 Color            : ATTRIBUTE3;
    
	// Motion blur input
	float4 MotionBlurData   : ATTRIBUTE4;    
    

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	float2	TexCoords0 : ATTRIBUTE5;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		float2	TexCoords1 : ATTRIBUTE6;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		float2	TexCoords2 : ATTRIBUTE7;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		float2	TexCoords3 : ATTRIBUTE8;
	#endif

	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4
		#error Too many texture coordinate sets defined on Geometry cache vertex input. Max: 4.
	#endif
#endif

	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

// RHI_RAYTRACING
#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input;
	
	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);
	uint VertexId = Tri.Indices[VertexIndex];
	
	Input.Position.x = GeomCacheMVF.Position[VertexId * 3 + 0];
	Input.Position.y = GeomCacheMVF.Position[VertexId * 3 + 1];
	Input.Position.z = GeomCacheMVF.Position[VertexId * 3 + 2];
	Input.TangentX = GeomCacheMVF.TangentX[VertexId].xyz;
	Input.TangentZ = GeomCacheMVF.TangentZ[VertexId];
	Input.Color = GeomCacheMVF.Color[VertexId];
    Input.MotionBlurData.x = GeomCacheMVF.MotionBlurData[VertexId * 3 + 0];
    Input.MotionBlurData.y = GeomCacheMVF.MotionBlurData[VertexId * 3 + 1];
    Input.MotionBlurData.z = GeomCacheMVF.MotionBlurData[VertexId * 3 + 2];
	
#if NUM_MATERIAL_TEXCOORDS_VERTEX
	float2 TexCoordXY = float2(GeomCacheMVF.TexCoords[VertexId * 2 + 0], GeomCacheMVF.TexCoords[VertexId * 2 + 1]);

	Input.TexCoords0 = TexCoordXY;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Input.TexCoords1 = TexCoordXY;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		Input.TexCoords2 = TexCoordXY;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		Input.TexCoords3 = TexCoordXY;
	#endif
#endif

	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	return Input;
}
#endif

struct FPositionOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

struct FPositionAndNormalOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	float4	Normal		: ATTRIBUTE1;
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(2)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

/** for depth-only pass */
float4 VertexFactoryGetWorldPosition(FPositionOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return TransformLocalToTranslatedWorld(Input.Position.xyz, SceneData.InstanceData.LocalToWorld);
}

/** for shadow pass (used for slope bias) */
float4 VertexFactoryGetWorldPosition(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return TransformLocalToTranslatedWorld(Input.Position.xyz, SceneData.InstanceData.LocalToWorld);
}

float3 VertexFactoryGetWorldNormal(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return RotateLocalToWorld(Input.Normal.xyz, SceneData.InstanceData.LocalToWorld, SceneData.InstanceData.InvNonUniformScale);
}

struct FVertexFactoryInterpolantsVSToPS
{
#if INTERPOLATE_VERTEX_COLOR
	float4 Color : COLOR0;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	// Pack interpolators to reduce the number of semantics used
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2]	: TEXCOORD0;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
#endif

	TANGENTTOWORLD_INTERPOLATOR_BLOCK
};

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS	
	UNROLL
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS;CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif
	Result.UnMirrored = TangentToWorld2.w;
#if INTERPOLATE_VERTEX_COLOR
	Result.VertexColor = Interpolants.Color;
#else
	Result.VertexColor = 0;
#endif
	Result.TwoSidedSign = 1;

#if VF_USE_PRIMITIVE_SCENE_DATA
	Result.PrimitiveId = Interpolants.PrimitiveId;
#endif

	return Result;
}

#define FBoneMatrix float3x4


// Cache data to avoid multiple calculation 
struct FVertexFactoryIntermediates
{
	// Unpacked position
	float3 UnpackedPosition;
	
	// Tangent Basis
	float3x3 TangentToLocal; 
	float3x3 TangentToWorld; 
    
	// Vertex Color
	float4 Color;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	Result.WorldPosition = WorldPosition;
	
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
		Result.PositionPrimitiveSpace = mul(float4(WorldPosition, 1), DFFastToTranslatedWorld(Intermediates.SceneData.Primitive.PreviousWorldToLocal, ResolvedView.PrevPreViewTranslation)).xyz;
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
		Result.PositionPrimitiveSpace = mul(float4(WorldPosition, 1), DFFastToTranslatedWorld(Intermediates.SceneData.Primitive.WorldToLocal, ResolvedView.PreViewTranslation)).xyz;
	}

	Result.VertexColor = Intermediates.Color;
	Result.TangentToWorld = Intermediates.TangentToWorld;
	Result.PreSkinnedPosition = Intermediates.UnpackedPosition.xyz;
	Result.PreSkinnedNormal = Input.TangentZ.xyz * 2.f - 1.f;

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	Result.TexCoords[0] = Input.TexCoords0;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Result.TexCoords[1] = Input.TexCoords1;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		Result.TexCoords[2] = Input.TexCoords2;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		Result.TexCoords[3] = Input.TexCoords3;
	#endif
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	Result.PrimitiveId = Intermediates.SceneData.PrimitiveId;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

	return Result;
}

/**
* Unpack position - uncompress xyz position to object space position
*/
float3 UnpackedPosition( FVertexFactoryInput Input )
{
	return float3(Input.Position.xyz * GeomCache.MeshExtension + GeomCache.MeshOrigin);
}

/**
* Derive tangent space matrix from vertex interpolants
*/
half3x3 CalcTangentToLocal(FVertexFactoryInput Input)
{
	half3x3 Result;
	
    // Unpack to -1 .. 1
	half3 TangentX = TangentBias(Input.TangentX);
	half4 TangentZ = TangentBias(Input.TangentZ);

	// derive the binormal by getting the cross product of the normal and tangent
	half3 TangentY = cross(TangentZ.xyz, TangentX) * TangentZ.w;
	
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized 8 bit vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	Result[0] = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	Result[1] = TangentY;
	Result[2] = TangentZ.xyz;

	return Result;
}

void CalcTangentToWorld(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, out float3 TangentToWorld0, out float4 TangentToWorld2)
{
	float3x3 LocalToWorld = DFToFloat3x3(Intermediates.SceneData.InstanceData.LocalToWorld);

	// Remove scaling.
	half3 InvScale = GetPrimitiveData(Intermediates).InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;

	float3x3 TangentToWorld = mul(Intermediates.TangentToLocal, LocalToWorld);

	TangentToWorld0 = TangentToWorld[0];
	TangentToWorld2 = float4(TangentToWorld[2], Input.TangentZ.w * GetPrimitive_DeterminantSign_FromFlags(GetPrimitiveData(Intermediates).Flags));
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.UnpackedPosition = UnpackedPosition(Input);

	// Fill TangentToLocal
	Intermediates.TangentToLocal = CalcTangentToLocal(Input);
	
	float3 TangentToWorld0;
	float4 TangentToWorld2;
	CalcTangentToWorld(Input, Intermediates, TangentToWorld0, TangentToWorld2);
	Intermediates.TangentToWorld = AssembleTangentToWorld(TangentToWorld0, TangentToWorld2);

	// Swizzle vertex color.
	Intermediates.Color = Input.Color FCOLOR_COMPONENT_SWIZZLE;

	return Intermediates;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToWorld[2];
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToLocal;
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformLocalToTranslatedWorld(Intermediates.UnpackedPosition, Intermediates.SceneData.InstanceData.LocalToWorld);
}

// local position relative to instance
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.UnpackedPosition;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

    //Packs the TangentToWorld matrix into the interpolants.    
	Interpolants.TangentToWorld0.w = 0;
	CalcTangentToWorld(Input, Intermediates, Interpolants.TangentToWorld0.xyz, Interpolants.TangentToWorld2);
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Interpolants.TangentToWorld2_Center = Interpolants.TangentToWorld2;
#endif

#if INTERPOLATE_VERTEX_COLOR
	Interpolants.Color = Intermediates.Color;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	Interpolants.PrimitiveId = Intermediates.SceneData.PrimitiveId;
#endif

	return Interpolants;
}

// @return The previous position of the vertex in object space.
float3 CalcPreviousPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
    float3 PreviousPosition =  (Input.MotionBlurData.xyz * GeomCache.MotionBlurDataExtension + GeomCache.MotionBlurDataOrigin) + Intermediates.UnpackedPosition * GeomCache.MotionBlurPositionScale;
#if 0    
    float ExaggerateMotion = 10.0;
    float3 Delta = Intermediates.UnpackedPosition - PreviousPosition;
    PreviousPosition = Intermediates.UnpackedPosition - Delta * ExaggerateMotion;
#endif    
    return PreviousPosition;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformPreviousLocalPositionToTranslatedWorld(CalcPreviousPosition(Input, Intermediates), GetPrimitiveData(Intermediates).PreviousLocalToWorld);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcPreviousPosition(Input, Intermediates);
}

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0,0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	
	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	FVertexFactoryRayTracingInterpolants O;

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for(int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++tc)
	{
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
	}
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.PrimitiveId = a.InterpolantsVSToPS.PrimitiveId;
#endif

	return O;
}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	return Interpolants.PrimitiveId;
#else // VF_USE_PRIMITIVE_SCENE_DATA
	return 0;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
}

#include "VertexFactoryDefaultInterface.ush"