// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollectionVertexFactory.ush: Shader code for FGeometryCollectionVertexFactory. 
=============================================================================*/

#include "SceneData.ush"
#include "VertexFactoryCommon.ush"
#include "LocalVertexFactoryCommon.ush"
#include "LightmapData.ush"

#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"

#ifndef MANUAL_VERTEX_FETCH
	#define MANUAL_VERTEX_FETCH 0
#endif

#ifndef USE_SHADER_BONE_TRANSFORM
	#define USE_SHADER_BONE_TRANSFORM (MANUAL_VERTEX_FETCH)
#endif

#define VF_ColorIndexMask_Index 0
#define VF_NumTexcoords_Index 1
#define FV_LightMapIndex_Index 2
#define VF_VertexOffset 3

#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	Buffer<float4> VertexFetch_BoneOriginBuffer;
	Buffer<float4> VertexFetch_BoneLightmapBuffer;

	#define VF_REQUIRES_HITPROXY_INDIRECTION 1
	int VertexFactoryGetVertexFetchParameter(int ParameterIndex)
	{
		return GeometryCollectionVF.VertexFetch_Parameters[ParameterIndex];
	}
#endif

// Since we don't have template functions it will have to be this way.
#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	#define GCVF_GET_INPUT_VERTEX_ID(InputStruct) ((InputStruct).VertexId)
#else // !MANUAL_VERTEX_FETCH
	#define GCVF_GET_INPUT_VERTEX_ID(InputStruct) (0U)
#endif


/**
 * Per-vertex inputs from bound vertex buffers
 */
struct FVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

#if !MANUAL_VERTEX_FETCH
	#if METAL_ES3_1_PROFILE
		float3	TangentX	: ATTRIBUTE1;
		// TangentZ.w contains sign of tangent basis determinant
		float4	TangentZ	: ATTRIBUTE2;

		float4	Color		: ATTRIBUTE3;
	#else
		half3	TangentX	: ATTRIBUTE1;
		// TangentZ.w contains sign of tangent basis determinant
		half4	TangentZ	: ATTRIBUTE2;

		half4	Color		: ATTRIBUTE3;
	#endif
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	#if !MANUAL_VERTEX_FETCH
		// These used to be packed texcoord arrays, but these cause problems with alighnment on some Vulkan drivers
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
			float4	TexCoords0 : ATTRIBUTE4;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 1
			float2	TexCoords0 : ATTRIBUTE4;
		#endif

		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
			float4	TexCoords1 : ATTRIBUTE5;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 3
			float2	TexCoords1 : ATTRIBUTE5;
		#endif

		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5
			float4	TexCoords2 : ATTRIBUTE6;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 5
			float2	TexCoords2 : ATTRIBUTE6;
		#endif

		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 7
			float4	TexCoords3 : ATTRIBUTE7;
		#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 7
			float2	TexCoords3 : ATTRIBUTE7;
		#endif
	#endif
#elif USE_PARTICLE_SUBUVS && !MANUAL_VERTEX_FETCH
	float2	TexCoords[1] : ATTRIBUTE4;
#endif
	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE13
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 

#if NEEDS_LIGHTMAP_COORDINATE && !MANUAL_VERTEX_FETCH
	float2	LightMapCoordinate : ATTRIBUTE15;
#endif

#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	uint VertexId : SV_VertexID;
#endif
};




/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-only shader.
 */
struct FPositionOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE1
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 

#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	uint VertexId : SV_VertexID;
#endif
};

/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-and-normal-only shader.
 */
struct FPositionAndNormalOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	float4	Normal		: ATTRIBUTE2;

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE1
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 

#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	uint VertexId : SV_VertexID;
#endif
};

/** 
 * Caches intermediates that would otherwise have to be computed multiple times.  Avoids relying on the compiler to optimize out redundant operations.
 */
struct FVertexFactoryIntermediates
{
	FDFMatrix LocalToWorld;
	FDFMatrix PrevLocalToWorld;

	half3x3 TangentToLocal;
	half3x3 TangentToWorld;
	half TangentToWorldSign;

	half4 Color;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

#if USE_PARTICLE_SUBUVS
	// Output TexCoord0 for when previewing materials that use ParticleSubUV.
	Result.Particle.SubUVCoords[0] = GetUV(Interpolants, 0);
	Result.Particle.SubUVCoords[1] = GetUV(Interpolants, 0);
#endif

	half3 TangentToWorld0 = GetTangentToWorld0(Interpolants).xyz;
	half4 TangentToWorld2 = GetTangentToWorld2(Interpolants);
	Result.UnMirrored = TangentToWorld2.w;

	Result.VertexColor = GetColor(Interpolants);

	// Required for previewing materials that use ParticleColor
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

#if LIGHTMAP_UV_ACCESS && NEEDS_LIGHTMAP_COORDINATE
	#if (ES3_1_PROFILE)
		// Not supported in pixel shader
		Result.LightmapUVs = float2(0, 0);
	#else
		Result.LightmapUVs = Interpolants.LightMapCoordinate.xy;
	#endif
#endif

	Result.TwoSidedSign = 1;
	Result.PrimitiveId = GetPrimitiveId(Interpolants);

#if NEEDS_PARTICLE_LOCAL_TO_WORLD
	Result.Particle.ParticleToWorld = GetPrimitiveData(Result.PrimitiveId).LocalToWorld;
#endif

#if NEEDS_PARTICLE_WORLD_TO_LOCAL
	Result.Particle.WorldToParticle = GetPrimitiveData(Result.PrimitiveId).WorldToLocal;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;

	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.Color;
	Result.TangentToWorld = Intermediates.TangentToWorld;

	Result.PrevFrameLocalToWorld = Intermediates.PrevLocalToWorld;

	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = TangentToLocal[2];

	const uint NumFetchTexCoords = GeometryCollectionVF.VertexFetch_Parameters[VF_NumTexcoords_Index];
#if MANUAL_VERTEX_FETCH && NUM_MATERIAL_TEXCOORDS_VERTEX
	UNROLL
	for (uint CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		// Clamp coordinates to mesh's maximum as materials can request more than are available
		uint ClampedCoordinateIndex = min(CoordinateIndex, NumFetchTexCoords - 1);
		Result.TexCoords[CoordinateIndex] = GeometryCollectionVF.VertexFetch_TexCoordBuffer[NumFetchTexCoords * (GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + ClampedCoordinateIndex];
	}
#elif NUM_MATERIAL_TEXCOORDS_VERTEX
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
		Result.TexCoords[0] = Input.TexCoords0.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Result.TexCoords[1] = NumFetchTexCoords > 1 ? Input.TexCoords0.zw : Result.TexCoords[0];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		Result.TexCoords[2] = NumFetchTexCoords > 2 ? Input.TexCoords1.xy : Result.TexCoords[1];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		Result.TexCoords[3] = NumFetchTexCoords > 3 ? Input.TexCoords1.zw : Result.TexCoords[2];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4
		Result.TexCoords[4] = NumFetchTexCoords > 4 ? Input.TexCoords2.xy : Result.TexCoords[3];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5
		Result.TexCoords[5] = NumFetchTexCoords > 5 ? Input.TexCoords2.zw : Result.TexCoords[4];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 6
		Result.TexCoords[6] = NumFetchTexCoords > 6 ? Input.TexCoords3.xy : Result.TexCoords[5];
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 7
		Result.TexCoords[7] = NumFetchTexCoords > 7 ? Input.TexCoords3.zw : Result.TexCoords[6];
	#endif
#endif  //MANUAL_VERTEX_FETCH && NUM_MATERIAL_TEXCOORDS_VERTEX

#if NEEDS_PARTICLE_LOCAL_TO_WORLD
	Result.Particle.ParticleToWorld = GetPrimitiveData(Intermediates).LocalToWorld;
#endif

#if NEEDS_PARTICLE_WORLD_TO_LOCAL
	Result.Particle.WorldToParticle = GetPrimitiveData(Intermediates).WorldToLocal;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

half3x3 LoadTangentToLocal(FVertexFactoryInput Input, inout float TangentSignOut)
{
#if MANUAL_VERTEX_FETCH
	half3 TangentInputX = GeometryCollectionVF.VertexFetch_PackedTangentsBuffer[2 * (GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + 0].xyz;
	half4 TangentInputZ = GeometryCollectionVF.VertexFetch_PackedTangentsBuffer[2 * (GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + 1].xyzw;
#else
	half3 TangentInputX = Input.TangentX;
	half4 TangentInputZ = Input.TangentZ;
#endif

	half3 TangentX = TangentBias(TangentInputX);
	half4 TangentZ = TangentBias(TangentInputZ);

	TangentSignOut = TangentZ.w;

	// derive the binormal by getting the cross product of the normal and tangent
	half3 TangentY = cross(TangentZ.xyz, TangentX) * TangentZ.w;
	
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	half3x3 Result;
	Result[0] = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	Result[1] = TangentY;
	Result[2] = TangentZ.xyz;

	return Result;
}

half3x3 CalcTangentToWorld(half3 InvScale, half3x3 TangentToLocal, FDFMatrix LocalToWorld)
{
	half3x3 LocalToWorld3x3 = DFToFloat3x3(LocalToWorld);
	LocalToWorld3x3[0] *= InvScale.x;
	LocalToWorld3x3[1] *= InvScale.y;
	LocalToWorld3x3[2] *= InvScale.z;
	return mul(TangentToLocal, LocalToWorld3x3);
}

/**
 * Temporary info to make it possible to load transform for position only & position/normal paths.
 */
struct FTransformData
{
	FSceneDataIntermediates SceneData;

	FDFMatrix LocalToWorld;
	FDFMatrix PrevLocalToWorld;
	float3 InvScale;
};

FTransformData LoadTransformData(FSceneDataIntermediates SceneData, uint VertexId)
{
	FTransformData TransformData;
	TransformData.SceneData = SceneData;
	TransformData.InvScale = TransformData.SceneData.InstanceData.InvNonUniformScale;
	FDFMatrix InstanceLocalToWorld = TransformData.SceneData.InstanceData.LocalToWorld;
	FDFMatrix PrevInstanceLocalToWorld = TransformData.SceneData.InstanceData.PrevLocalToWorld;

#if MANUAL_VERTEX_FETCH || USE_SHADER_BONE_TRANSFORM
	uint BoneIndex = GCBoneLooseParameters.VertexFetch_BoneMapBuffer[GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + VertexId];
	float4x4 LocalToInstance = float4x4(
		float4(GCBoneLooseParameters.VertexFetch_BoneTransformBuffer[4 * BoneIndex + 0].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BoneTransformBuffer[4 * BoneIndex + 1].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BoneTransformBuffer[4 * BoneIndex + 2].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BoneTransformBuffer[4 * BoneIndex + 3].xyz, 1.0f));

	TransformData.LocalToWorld = DFMultiply(LocalToInstance, InstanceLocalToWorld);

	float4x4 PrevLocalToInstance = float4x4(
		float4(GCBoneLooseParameters.VertexFetch_BonePrevTransformBuffer[4 * BoneIndex + 0].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BonePrevTransformBuffer[4 * BoneIndex + 1].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BonePrevTransformBuffer[4 * BoneIndex + 2].xyz, 0.0f),
		float4(GCBoneLooseParameters.VertexFetch_BonePrevTransformBuffer[4 * BoneIndex + 3].xyz, 1.0f));

	TransformData.PrevLocalToWorld = DFMultiply(PrevLocalToInstance, PrevInstanceLocalToWorld);
#else
	// In this case the LTW because it already is skinned on CPU
	// Note: this changes the meaning of tangent to local etc, I assume that is fine because that is how it used to be
	TransformData.LocalToWorld = InstanceLocalToWorld;
	TransformData.PrevLocalToWorld = InstanceLocalToWorld;
#endif
	return TransformData;
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);

	FTransformData TransformData = LoadTransformData(Intermediates.SceneData, GCVF_GET_INPUT_VERTEX_ID(Input));
	float DeterminantSign = Intermediates.SceneData.InstanceData.DeterminantSign;

	Intermediates.LocalToWorld = TransformData.LocalToWorld;
	Intermediates.PrevLocalToWorld = TransformData.PrevLocalToWorld;

#if MANUAL_VERTEX_FETCH
	Intermediates.Color = GeometryCollectionVF.VertexFetch_ColorComponentsBuffer[(GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) & GeometryCollectionVF.VertexFetch_Parameters[VF_ColorIndexMask_Index]] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
#else
	Intermediates.Color = Input.Color FCOLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
#endif

	float TangentSign = 1.0;
	Intermediates.TangentToLocal = LoadTangentToLocal(Input, TangentSign);
	Intermediates.TangentToWorld = CalcTangentToWorld(TransformData.InvScale, Intermediates.TangentToLocal, Intermediates.LocalToWorld);

	Intermediates.TangentToWorldSign = TangentSign * DeterminantSign;

	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
half3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.TangentToLocal;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformLocalToTranslatedWorld(Input.Position.xyz, Intermediates.LocalToWorld);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);
	
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}

#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 0 && USE_PARTICLE_SUBUVS
	#if MANUAL_VERTEX_FETCH
		SetUV(Interpolants, 0, GeometryCollectionVF.VertexFetch_TexCoordBuffer[GeometryCollectionVF.VertexFetch_Parameters[VF_NumTexcoords_Index] * (GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId)]);
	#else
		SetUV(Interpolants, 0, Input.TexCoords[0]);
	#endif
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	float2 LightMapCoordinate = 0;
	float2 ShadowMapCoordinate = 0;
	#if MANUAL_VERTEX_FETCH
		float2 LightMapCoordinateInput = GeometryCollectionVF.VertexFetch_TexCoordBuffer[GeometryCollectionVF.VertexFetch_Parameters[VF_NumTexcoords_Index] * (GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + GeometryCollectionVF.VertexFetch_Parameters[FV_LightMapIndex_Index]];
	#else
		float2 LightMapCoordinateInput = Input.LightMapCoordinate;
	#endif

	uint LightmapDataIndex = GetPrimitiveData(Intermediates).LightmapDataIndex + GeometryCollectionVF.LODLightmapDataIndex;
	uint LightmapUVIndex = GetPrimitiveData(Intermediates).LightmapUVIndex;

	float4 LightMapCoordinateScaleBias = VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex);
	LightMapCoordinate = LightMapCoordinateInput * LightMapCoordinateScaleBias.xy + LightMapCoordinateScaleBias.zw;

#if STATICLIGHTING_TEXTUREMASK
	float4 ShadowMapCoordinateScaleBias = VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(Input, LightmapDataIndex);
	ShadowMapCoordinate = LightMapCoordinateInput * ShadowMapCoordinateScaleBias.xy + ShadowMapCoordinateScaleBias.zw;
#endif

	SetLightMapCoordinate(Interpolants, LightMapCoordinate, ShadowMapCoordinate);
	SetLightmapDataIndex(Interpolants, LightmapDataIndex);

#endif	// NEEDS_LIGHTMAP_COORDINATE

	SetTangents(Interpolants, Intermediates.TangentToWorld[0], Intermediates.TangentToWorld[2], Intermediates.TangentToWorldSign);
	SetColor(Interpolants, Intermediates.Color);

	SetSceneData(Interpolants, Intermediates.SceneData);

	return Interpolants;
}

/** for depth-only pass */
float4 VertexFactoryGetWorldPosition(FPositionOnlyVertexFactoryInput Input)
{
	FTransformData TransformData = LoadTransformData(VF_GPUSCENE_GET_INTERMEDIATES(Input), GCVF_GET_INPUT_VERTEX_ID(Input));
	return TransformLocalToTranslatedWorld(Input.Position.xyz, TransformData.LocalToWorld);
}

/** for depth-only pass (slope depth bias) */
float4 VertexFactoryGetWorldPosition(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FTransformData TransformData = LoadTransformData(VF_GPUSCENE_GET_INTERMEDIATES(Input), GCVF_GET_INPUT_VERTEX_ID(Input));
	return TransformLocalToTranslatedWorld(Input.Position.xyz, TransformData.LocalToWorld);
}

float3 VertexFactoryGetWorldNormal(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FTransformData TransformData = LoadTransformData(VF_GPUSCENE_GET_INTERMEDIATES(Input), GCVF_GET_INPUT_VERTEX_ID(Input));
	return RotateLocalToWorld(Input.Position.xyz, TransformData.LocalToWorld, TransformData.InvScale).xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToWorld[2];
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformPreviousLocalPositionToTranslatedWorld(Input.Position.xyz, Intermediates.PrevLocalToWorld);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

#if RAYHITGROUPSHADER

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

float2 VertexFactoryGetRayTracingTextureCoordinate(FVertexFactoryRayTracingInterpolants Interpolants)
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0, 0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;

	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);

	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	// Default initialize. Otherwise, some graphics pipelines that
	// couple tessellation with geometry shaders won't write to all TEXCOORD semantics,
	// but read from them when <FVertexFactoryRayTracingInterpolants> is being copied as a whole.
	FVertexFactoryRayTracingInterpolants O = (FVertexFactoryRayTracingInterpolants)0;

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.InstanceId = a.InterpolantsVSToPS.InstanceId;
#if NEEDS_LIGHTMAP_COORDINATE
	O.InterpolantsVSToPS.LightmapDataIndex = a.InterpolantsVSToPS.LightmapDataIndex;
#endif
#endif	
	// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
	// interpolated normal from 'whatever' interpolation scheme we're using

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif
#if	NEEDS_PER_INSTANCE_PARAMS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.PerInstanceParams);
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapCoordinate);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
		for (int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++tc)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
		}
#elif USE_PARTICLE_SUBUVS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[0]);
#endif

	return O;
}
#endif


#if RAYHITGROUPSHADER

FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);

	Input.VertexId = Tri.Indices[VertexIndex];
	Input.Position = float4(Tri.LocalPositions[VertexIndex], 1.0f);

	// Note: GetInstanceUserData() stores the GPU-Scene instance ID
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	return Input;
}

#endif

#if RAYHITGROUPSHADER || COMPUTESHADER
uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif

#if COMPUTESHADER  
FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

#if MANUAL_VERTEX_FETCH
	Input.VertexId = TriangleIndex * 3 + VertexIndex;
	uint VertexOffset = GeometryCollectionVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId * 3;
	Input.Position.x = GeometryCollectionVF.VertexFetch_PositionBuffer[VertexOffset + 0];
	Input.Position.y = GeometryCollectionVF.VertexFetch_PositionBuffer[VertexOffset + 1];
	Input.Position.z = GeometryCollectionVF.VertexFetch_PositionBuffer[VertexOffset + 2];
#endif

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, PrimitiveData.InstanceSceneDataOffset + DrawInstanceId, DrawInstanceId);

	return Input;
}

#endif

#undef GCVF_GET_INPUT_VERTEX_ID

#include "VertexFactoryDefaultInterface.ush"
