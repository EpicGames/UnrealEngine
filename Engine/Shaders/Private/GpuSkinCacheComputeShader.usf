// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GpuSkinCacheComputeShader.usf: Perform vertex skinning into a buffer to avoid skinning in the vertex shader.
=============================================================================*/

#include "Common.ush"
#include "Definitions.usf"
#include "SceneData.ush"
#include "VertexFactoryCommon.ush"
#include "GpuSkinCommon.ush"
#include "DynamicMeshBounds.ush"
#include "WaveOpUtil.ush"

// 0/1 setup by CPU
// #define GPUSKIN_APEX_CLOTH

// 0/1 setup by CPU
// #define GPUSKIN_MORPH_BLEND

// Set to 1 to enable passing through values from vertex buffer (ie do not skin)
#define GPUSKIN_DEBUG_PASSTHROUGH	0

#define FBoneMatrix float3x4

STRONG_TYPE FCompressedBoneTransformBuffer BoneMatrices;

#if GPUSKIN_MORPH_BLEND
// float3 DeltaPosition, PackedNormal, see FMorphGPUSkinVertex
STRONG_TYPE Buffer<float> MorphBuffer;
// data offset to start indexing the data in MorphBuffer, in float units
uint MorphBufferOffset;
	
void GetMorphBuffer(uint VertexIndex, out float3 DeltaPosition, out half3 DeltaTangentZ)
 {
	// see CPU code: "check(MorphStride == sizeof(float) * 6);"
	// in floats
	// 3 + 3 floats because of the format defined in FMorphGPUSkinVertex
	uint Offset = VertexIndex * (3 + 3);

	DeltaPosition = float3(
		MorphBuffer[Offset + 0],
		MorphBuffer[Offset + 1],
		MorphBuffer[Offset + 2]);

	DeltaTangentZ = half3(
		MorphBuffer[Offset + 3],
		MorphBuffer[Offset + 4],
		MorphBuffer[Offset + 5]);
}
#endif

#if GPUSKIN_APEX_CLOTH
// This must match NUM_INFLUENCES_PER_VERTEX in ClothingMeshUtils.cpp and GpuSkinVertexFactory.ush
// It represents the maximum number of influences so perhaps should be renamed. Right now we are keeping the name matching for easy search.
// TODO: Make this easier to change in without messing things up
#define NUM_INFLUENCES_PER_VERTEX 5
uint ClothNumInfluencesPerVertex;

struct FVertexTriangleInfluence
{
    float4  PositionBaryCoordsAndDist;
    float4  NormalBaryCoordsAndDist;
    float4  TangentBaryCoordsAndDist;
    uint4   SourceMeshVertIndices;
    float   Weight;
};

// In ClothBuffer:
#define NUM_FLOAT4S_PER_VERTEX_INFLUENCE 4;

Buffer<float4> ClothBuffer;
Buffer<float2> ClothPositionsAndNormalsBuffer;
uint ClothBufferOffset;
float ClothBlendWeight;
/** Transform from cloth space (relative to cloth root bone) to local/component space */
float4x4 ClothToLocal;
/** Scale of the owner actor */
float3 WorldScale;

void GetClothBuffer(uint VertexIndex, uint ClothIndexOffset, out FVertexTriangleInfluence Influences[NUM_INFLUENCES_PER_VERTEX])
{
	const uint VertexOffset = ClothNumInfluencesPerVertex * VertexIndex + ClothIndexOffset;
	UNROLL_N(NUM_INFLUENCES_PER_VERTEX)
	for (int i = 0; i < ClothNumInfluencesPerVertex; ++i )
	{
		const uint Offset = (VertexOffset + i) * NUM_FLOAT4S_PER_VERTEX_INFLUENCE;

        Influences[i].PositionBaryCoordsAndDist = ClothBuffer[Offset];
        Influences[i].NormalBaryCoordsAndDist = ClothBuffer[Offset + 1];
        Influences[i].TangentBaryCoordsAndDist = ClothBuffer[Offset + 2];

	    uint4 PackedIndices = asuint(ClothBuffer[Offset + 3]);
        Influences[i].SourceMeshVertIndices.yw = (PackedIndices.xy >> 16) & 0xffff;
        Influences[i].SourceMeshVertIndices.xz = PackedIndices.xy & 0xffff;

        Influences[i].Weight = asfloat(PackedIndices[2]);
    }
}
#endif

struct FVertexUnpacked
{
	half4	TangentX;
	half4	TangentZ;

#if !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	FGPUSkinIndexAndWeight IndexAndWeights;
#endif // GPUSKIN_UNLIMITED_BONE_INFLUENCE

#if GPUSKIN_MORPH_BLEND
	// morph target, added to the Position
	float3	DeltaPosition;
	// morph target, added to the TangentZ and then used to derive new TangentX and TangentY, -2..2
	half3	DeltaTangentZ;
#endif

#if GPUSKIN_APEX_CLOTH
    FVertexTriangleInfluence Influences[NUM_INFLUENCES_PER_VERTEX];
#endif
};

struct FVertexFactoryIntermediates
{
	// Blend Matrix (used for position/tangents)
	FBoneMatrix BlendMatrix;

	// Unpacked position (includes DeltaPosition if GPUSKIN_MORPH_BLEND)
	float3 UnpackedPosition;
	
	// Tangent Basis (includes DeltaTangentZ if GPUSKIN_MORPH_BLEND)
	half3x3 TangentToLocal; 

#if GPUSKIN_APEX_CLOTH
    float3 SimulatedPosition;
#endif
};

#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
// Bits 0-7 => Size of the bone weight index in bytes / bits 8-15 => Size of the bone weight weights value in bytes
uint InputWeightIndexSize;	
Buffer<uint> InputWeightLookupStream;
#endif
Buffer<uint> InputWeightStream;

Buffer<SNORM float4> TangentInputBuffer;
RWBuffer<TANGENT_RWBUFFER_FORMAT> TangentBufferUAV;
Buffer<float> PositionInputBuffer;
RWBuffer<float> PositionBufferUAV;

uint InputStreamStart;
uint SkinCacheStart;
uint NumVertices;

uint InputWeightStart;
uint InputWeightStride;

#if ENABLE_DYNAMIC_MESH_BOUNDS
RWStructuredBuffer<int4> OutBoundsBufferUAV;
int DynamicBoundsOffset;
#endif

FVertexUnpacked UnpackedVertex(uint MorphIndex, uint ClothIndex, uint ClothIndexOffset, uint VertexIndex, uint WeightOffset)
{
	FVertexUnpacked Unpacked;

	Unpacked.TangentX = TangentBias_SkinCache(TangentInputBuffer[2 * VertexIndex + GPUSKIN_VB_OFFSET_TANGENT_X]);
	Unpacked.TangentZ = TangentBias_SkinCache(TangentInputBuffer[2 * VertexIndex + GPUSKIN_VB_OFFSET_TANGENT_Z]);

#if !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	Unpacked.IndexAndWeights = GetBlendIndicesAndWeights(InputWeightStream, WeightOffset);
#endif

#if GPUSKIN_MORPH_BLEND
	GetMorphBuffer(MorphIndex, Unpacked.DeltaPosition, Unpacked.DeltaTangentZ);
#endif

#if GPUSKIN_APEX_CLOTH
    GetClothBuffer(ClothIndex, ClothIndexOffset, Unpacked.Influences);
#endif
	return Unpacked;
}

/** transform position by weighted sum of skinning matrices */
float3 SkinPosition( FVertexFactoryIntermediates Intermediates )
{
	float4 Position = float4(Intermediates.UnpackedPosition,1);

	// Note the use of mul(Matrix,Vector), bone matrices are stored transposed
	// for tighter packing.
	return mul( Intermediates.BlendMatrix, Position );
}

[numthreads(64,1,1)]
void SkinCacheUpdateBatchCS(uint3 GroupID : SV_GroupID,
 uint3 DispatchThreadID : SV_DispatchThreadID,
 uint3 GroupThreadID : SV_GroupThreadID)
{
	uint VertexIndex = DispatchThreadID.x;
	if (VertexIndex >= NumVertices)
	{
		return;
	}

	// Find offset for regular (position/tangent/UV) vertex buffer
	uint InputOffset = InputStreamStart + VertexIndex;
	uint OutputOffset = SkinCacheStart + VertexIndex;

	// Find offset for skin weight buffer
	uint WeightOffset = InputWeightStart + (VertexIndex * (InputWeightStride/4));

#if GPUSKIN_MORPH_BLEND
#if GPUSKIN_APEX_CLOTH
	FVertexUnpacked Unpacked = UnpackedVertex(MorphBufferOffset + VertexIndex, VertexIndex, ClothBufferOffset, InputOffset, WeightOffset);
#else
	FVertexUnpacked Unpacked = UnpackedVertex(MorphBufferOffset + VertexIndex, VertexIndex, 0, InputOffset, WeightOffset);
#endif
#else
#if GPUSKIN_APEX_CLOTH
	FVertexUnpacked Unpacked = UnpackedVertex(VertexIndex, VertexIndex, ClothBufferOffset, InputOffset, WeightOffset);
#else
	FVertexUnpacked Unpacked = UnpackedVertex(VertexIndex, VertexIndex, 0, InputOffset, WeightOffset);
#endif
#endif

	// Perform the skinning
	FVertexFactoryIntermediates	Intermediates = (FVertexFactoryIntermediates)0;
	
#if 0
	// Test no blend mtx
	Intermediates.BlendMatrix[0] = float4(1,0,0,0);
	Intermediates.BlendMatrix[1] = float4(0,1,0,0);
	Intermediates.BlendMatrix[2] = float4(0,0,1,0);
#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	Intermediates.BlendMatrix = ComputeBoneMatrixWithUnlimitedInfluences(BoneMatrices, InputWeightStream, InputWeightIndexSize, InputWeightLookupStream[InputOffset]);
#else
	Intermediates.BlendMatrix = ComputeBoneMatrixWithLimitedInfluences(BoneMatrices, Unpacked.IndexAndWeights);
#endif

	Intermediates.UnpackedPosition.x = PositionInputBuffer[InputOffset * 3 + 0];
	Intermediates.UnpackedPosition.y = PositionInputBuffer[InputOffset * 3 + 1];
	Intermediates.UnpackedPosition.z = PositionInputBuffer[InputOffset * 3 + 2];

	half3 LocalTangentX = Unpacked.TangentX.xyz;
	half3 LocalTangentZ = Unpacked.TangentZ.xyz;

#if GPUSKIN_MORPH_BLEND
	{
		Intermediates.UnpackedPosition += Unpacked.DeltaPosition;

		// calc new normal by offseting it with the delta
		LocalTangentZ = normalize( LocalTangentZ + Unpacked.DeltaTangentZ);
		// derive the new tangent by orthonormalizing the new normal against
		// the base tangent vector (assuming these are normalized)
		LocalTangentX = normalize( LocalTangentX - (dot(LocalTangentX, LocalTangentZ) * LocalTangentZ) );
	}
#else
#if GPUSKIN_APEX_CLOTH

    float3 ClothTangentX = float3(0,0,0);
    float3 ClothTangentZ = float3(0,0,0);

    Intermediates.SimulatedPosition = float3(0,0,0);
    float3 NormalPosition = float3(0,0,0);
    float3 TangentPosition = float3(0,0,0);
    int NumInfluences = 0;
	float SumWeights = 0.0;
	float ClothWeight = 0.0;

	const float3 WorldScaleAbs = abs(WorldScale);  // World scale can't be used mirrored to calculate the clothing positions and tangents since the cloth normals are then reversed

	UNROLL_N(NUM_INFLUENCES_PER_VERTEX)
    for (int i = 0; i < ClothNumInfluencesPerVertex; ++i )
    {
        const FVertexTriangleInfluence Influence = Unpacked.Influences[i];

	    if( Influence.SourceMeshVertIndices.w < 0xFFFF )
	    { 
            ++NumInfluences;

		    float3 A = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3], ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 1].x);
		    float3 B = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3], ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 1].x);
		    float3 C = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3], ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 1].x);

		    float3 NA = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 1].y, ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 2]);
		    float3 NB = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 1].y, ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 2]);
		    float3 NC = float3(ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 1].y, ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 2]);

	    	ClothWeight += ClothBlendWeight * (1.0f - (Influence.SourceMeshVertIndices.w / 65535.0f));

			float Weight = 1.0f;
			if ( ClothNumInfluencesPerVertex > 1 )
			{
				// Weight is packed in the last coordinate
				Weight = Influence.Weight;
				SumWeights += Weight;
			}
			else
			{
				// Single influence, weight is 1.0
	            Weight = 1.0f;
                SumWeights = 1.0f;	   
			}

			NormalPosition += Weight * (Influence.NormalBaryCoordsAndDist.x * (A + NA * Influence.NormalBaryCoordsAndDist.w * WorldScaleAbs.x)
								      + Influence.NormalBaryCoordsAndDist.y * (B + NB * Influence.NormalBaryCoordsAndDist.w * WorldScaleAbs.y)
								      + Influence.NormalBaryCoordsAndDist.z * (C + NC * Influence.NormalBaryCoordsAndDist.w * WorldScaleAbs.z));

		    TangentPosition += Weight * (Influence.TangentBaryCoordsAndDist.x * (A + NA * Influence.TangentBaryCoordsAndDist.w * WorldScaleAbs.x)
								       + Influence.TangentBaryCoordsAndDist.y * (B + NB * Influence.TangentBaryCoordsAndDist.w * WorldScaleAbs.y)
								       + Influence.TangentBaryCoordsAndDist.z * (C + NC * Influence.TangentBaryCoordsAndDist.w * WorldScaleAbs.z));

			float3 TriangleBary = float3(Influence.PositionBaryCoordsAndDist.x,
                                         Influence.PositionBaryCoordsAndDist.y,
										 1.0f - Influence.PositionBaryCoordsAndDist.x - Influence.PositionBaryCoordsAndDist.y);

            float3 SimPosition = TriangleBary.x*(A+NA*Influence.PositionBaryCoordsAndDist.w * WorldScaleAbs.x)
					           + TriangleBary.y*(B+NB*Influence.PositionBaryCoordsAndDist.w * WorldScaleAbs.y)
					           + TriangleBary.z*(C+NC*Influence.PositionBaryCoordsAndDist.w * WorldScaleAbs.z);

	        Intermediates.SimulatedPosition += Weight * SimPosition;
	    }
    }

    if ( NumInfluences > 0 && SumWeights > 1e-4f )
    {
		float InvWeight = 1.0f / SumWeights; 
        Intermediates.SimulatedPosition *= InvWeight;
        TangentPosition *= InvWeight;
        NormalPosition *= InvWeight;

        ClothTangentX = normalize(TangentPosition - Intermediates.SimulatedPosition);
        ClothTangentZ = normalize(NormalPosition - Intermediates.SimulatedPosition);

	    // Simulated cloth data are in cloth space so need to change into local space
	    ClothTangentX = mul(ClothTangentX, (half3x3)ClothToLocal);
	    ClothTangentZ = mul(ClothTangentZ, (half3x3)ClothToLocal);
    }
	else
	{
		Intermediates.SimulatedPosition = float3(0, 0, 0);
	}
	
	if (ClothNumInfluencesPerVertex > 1)
	{
		ClothWeight /= ClothNumInfluencesPerVertex;
	}
#endif // GPUSKIN_APEX_CLOTH
#endif // GPUSKIN_MORPH_BLEND

	float3 SPos = SkinPosition(Intermediates);

#if GPUSKIN_APEX_CLOTH
	// Transform simulated position from cloth space into local space and blend with skinned position
    float4 TransformedSimulatedPos = mul(float4(Intermediates.SimulatedPosition.xyz, 1), ClothToLocal);
    SPos = lerp(SPos.xyz, TransformedSimulatedPos.xyz, ClothWeight);
#endif

	half3 TangentX = normalize(mul((half3x3)Intermediates.BlendMatrix, LocalTangentX));
	half3 TangentZ = normalize(mul((half3x3)Intermediates.BlendMatrix, LocalTangentZ));
	
#if GPUSKIN_APEX_CLOTH
    TangentX = ClothTangentX * ClothWeight + TangentX * (1.0f - ClothWeight);
    TangentZ = ClothTangentZ * ClothWeight + TangentZ * (1.0f - ClothWeight);
#endif

	PositionBufferUAV[OutputOffset * 3 + 0] = SPos.x;
	PositionBufferUAV[OutputOffset * 3 + 1] = SPos.y;
	PositionBufferUAV[OutputOffset * 3 + 2] = SPos.z;

	// If wave vote is not supported this just hits atomic operations directly, which is slow.
#if ENABLE_DYNAMIC_MESH_BOUNDS && COMPILER_SUPPORTS_WAVE_VOTE
	if (DynamicBoundsOffset >= 0)
	{
		WaveInterlockedMin(OutBoundsBufferUAV[DynamicBoundsOffset * 2].x, int(floor(SPos.x)));
		WaveInterlockedMin(OutBoundsBufferUAV[DynamicBoundsOffset * 2].y, int(floor(SPos.y)));
		WaveInterlockedMin(OutBoundsBufferUAV[DynamicBoundsOffset * 2].z, int(floor(SPos.z)));
		
		WaveInterlockedMax(OutBoundsBufferUAV[DynamicBoundsOffset * 2 + 1].x, int(ceil(SPos.x)));
		WaveInterlockedMax(OutBoundsBufferUAV[DynamicBoundsOffset * 2 + 1].y, int(ceil(SPos.y)));
		WaveInterlockedMax(OutBoundsBufferUAV[DynamicBoundsOffset * 2 + 1].z, int(ceil(SPos.z)));
	}
#endif
	TangentBufferUAV[2 * OutputOffset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_X] = TangentUnbias_SkinCache(half4(TangentX, Unpacked.TangentX.w));
	TangentBufferUAV[2 * OutputOffset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z] = TangentUnbias_SkinCache(half4(TangentZ, Unpacked.TangentZ.w));

#if GPUSKIN_DEBUG_PASSTHROUGH
	// Passthrough debug code
	PositionBufferUAV[OutputOffset * 3 + 0] = PositionInputBuffer[InputOffset * 3 + 0];
	PositionBufferUAV[OutputOffset * 3 + 1] = PositionInputBuffer[InputOffset * 3 + 1];
	PositionBufferUAV[OutputOffset * 3 + 2] = PositionInputBuffer[InputOffset * 3 + 2];
	TangentBufferUAV[2 * OutputOffset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_X] = TangentInputBuffer[2 * InputOffset + GPUSKIN_VB_OFFSET_TANGENT_X];
	TangentBufferUAV[2 * OutputOffset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z] = TangentInputBuffer[2 * InputOffset + GPUSKIN_VB_OFFSET_TANGENT_Z];
#endif
}