// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GpuSkinCacheCommon.usf: Common properties and defines for GPU Skinning Cache
=============================================================================*/

#pragma once

#include "Skinning/SkinningDataDecode.ush"

// These offsets are in uint32/floats; they attempt to match TGPUSkinVertexBase<> & derived classes:
//	TGPUSkinVertexBase
//	TGPUSkinVertexFloat16Uvs<1..N>
//	TGPUSkinVertexFloat32Uvs<1..N>

// As we know the properties of the Vertices, the layout looks like this (offset, size in u32):
//	MEMBER				TGPUSkinVertexBase	TGPUSkinVertexFloat16Uvs<1..N>	TGPUSkinVertexFloat32Uvs<1..N>
//	TangentX			0,1					0,1								0,1
//	TangentZ			1,1					1,1								1,1
//	Position			2,3					2,3								2,3
//	UVs[1..N]			N/A					5,1..N							5,(1..N)*2

#define	GPUSKIN_VB_OFFSET_TANGENT_X			0
#define	GPUSKIN_VB_OFFSET_TANGENT_Z			1

#ifndef GPUSKIN_USE_EXTRA_INFLUENCES
#define GPUSKIN_USE_EXTRA_INFLUENCES		0
#endif

#ifndef GPUSKIN_BONE_INDEX_UINT16
#define GPUSKIN_BONE_INDEX_UINT16			0	// 16-bit bone index if 1, otherwise 8-bit bone index
#endif

#ifndef GPUSKIN_BONE_WEIGHTS_UINT16
#define GPUSKIN_BONE_WEIGHTS_UINT16			0	// 16-bit bone weights if 1, otherwise 8-bit bone weights
#endif

#ifndef GPUSKIN_UNLIMITED_BONE_INFLUENCE
#define GPUSKIN_UNLIMITED_BONE_INFLUENCE	0
#endif

#define	GPUSKIN_VB_OFFSET_INFLUENCEBONES	0
#define	GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS	((1 + GPUSKIN_USE_EXTRA_INFLUENCES) * (1 + GPUSKIN_BONE_INDEX_UINT16))


#ifndef GPUSKIN_RWBUFFER_OFFSET_TANGENT_X
#define GPUSKIN_RWBUFFER_OFFSET_TANGENT_X	0
#endif

#ifndef GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z
#define GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z	1
#endif

#ifndef GPUSKIN_LIMIT_2BONE_INFLUENCES
#define GPUSKIN_LIMIT_2BONE_INFLUENCES 0
#endif

// @return 0..255
uint4 UnpackU8x4(uint Packed)
{
	uint4 Unpacked;
	Unpacked.x = (Packed >> 0) & 0xff;
	Unpacked.y = (Packed >> 8) & 0xff;
	Unpacked.z = (Packed >> 16) & 0xff;
	Unpacked.w = (Packed >> 24) & 0xff;
	return Unpacked;
}

// @return 0..1
half4 UnpackU8x4N(uint Packed)
{
	return UnpackU8x4(Packed) / 255.0f;
}

// @return -1..1
half4 UnpackS8x4N(uint Packed)
{
	return UnpackU8x4(Packed) / 127.5f - 1;
}

// @param Unpacked -1..1
// @return layout 0xWWZZYYXX where 0..0xff is mapped to -1..1
uint PackS8x4N(half4 Unpacked)
{
	uint Packed = asuint(
		(uint((Unpacked.x + 1.0f) * 127.499f) << 0) |
		(uint((Unpacked.y + 1.0f) * 127.499f) << 8) |
		(uint((Unpacked.z + 1.0f) * 127.499f) << 16) |
		(uint((Unpacked.w + 1.0f) * 127.499f) << 24));
	return Packed;
}

// @param Unpacked -1..1
// @return layout 0x00ZZYYXX where 0..0xff is mapped to -1..1
uint PackS8x3N(half3 Unpacked)
{
	uint Packed = asuint(
		(uint((Unpacked.x + 1.0f) * 127.499f) << 0) |
		(uint((Unpacked.y + 1.0f) * 127.499f) << 8) |
		(uint((Unpacked.z + 1.0f) * 127.499f) << 16));
	return Packed;
}

#if OPENGL_PROFILE
// GL does not support SNORM images, so instead we write to SINT
#define TANGENT_RWBUFFER_FORMAT		int4
#define TangentBias_SkinCache(X)	ConvertTangentUnormToSnorm8(X)
#define TangentUnbias_SkinCache(X)	round(X * 32767.0)
#else
#define TANGENT_RWBUFFER_FORMAT		SNORM float4
#define TangentBias_SkinCache(X)	(X)
#define TangentUnbias_SkinCache(X)	(X)
#endif // OPENGL_PROFILE

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers for accessing vertex indices and weights

int4 GetBlendIndices(Buffer<uint> InWeightBuffer, uint StreamOffset, uint ExtraInfluenceIndex)
{
	int4 UnpackedBlendIndices = 0;

#if GPUSKIN_BONE_INDEX_UINT16
	StreamOffset += GPUSKIN_VB_OFFSET_INFLUENCEBONES + 2 * ExtraInfluenceIndex;
	UnpackedBlendIndices.xy = UnpackUInt2FromUInt(InWeightBuffer[StreamOffset   ]);
	UnpackedBlendIndices.zw = UnpackUInt2FromUInt(InWeightBuffer[StreamOffset + 1]);
#else // GPUSKIN_BONE_INDEX_UINT16
	StreamOffset += GPUSKIN_VB_OFFSET_INFLUENCEBONES + ExtraInfluenceIndex;
	UnpackedBlendIndices = UnpackUintToUint4(InWeightBuffer[StreamOffset]);
#endif // GPUSKIN_BONE_INDEX_UINT16
	return UnpackedBlendIndices;
}

half4 GetBlendWeights(Buffer<uint> InWeightBuffer, uint StreamOffset, uint ExtraInfluenceIndex, uint InNumBoneInfluences)
{
	StreamOffset += ((InNumBoneInfluences + 3) / 4) * (1 + GPUSKIN_BONE_INDEX_UINT16);

	half4 UnpackedBlendWeights = 0;
#if GPUSKIN_BONE_WEIGHTS_UINT16
	StreamOffset += ExtraInfluenceIndex * 2;
	UnpackedBlendWeights.xy = UnpackUnorm2x16(InWeightBuffer[StreamOffset   ]);
	UnpackedBlendWeights.zw = UnpackUnorm2x16(InWeightBuffer[StreamOffset + 1]);	
#else
	StreamOffset += ExtraInfluenceIndex;
	UnpackedBlendWeights = UnpackRGBA8(InWeightBuffer[StreamOffset]);
#endif
	return UnpackedBlendWeights;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Retrieve vertex indices & weights and compute bone matrix

struct FGPUSkinIndexAndWeight
{
	int4 BlendIndices;
	int4 BlendIndices2;
	half4 BlendWeights;
	half4 BlendWeights2;
};

FGPUSkinIndexAndWeight GetBlendIndicesAndWeights(Buffer<uint> InWeightBuffer, uint InWeightOffset)
{
	FGPUSkinIndexAndWeight Out = (FGPUSkinIndexAndWeight)0;

	// Indices
#if GPUSKIN_BONE_INDEX_UINT16
	Out.BlendIndices.xy = UnpackUInt2FromUInt(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES    ]);
	Out.BlendIndices.zw = UnpackUInt2FromUInt(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES + 1]);
	#if GPUSKIN_USE_EXTRA_INFLUENCES
	Out.BlendIndices2.xy = UnpackUInt2FromUInt(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES + 2]);
	Out.BlendIndices2.zw = UnpackUInt2FromUInt(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES + 3]);
	#endif
#else // GPUSKIN_BONE_INDEX_UINT16
	Out.BlendIndices  = UnpackUintToUint4(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES]);
	#if GPUSKIN_USE_EXTRA_INFLUENCES
	Out.BlendIndices2 = UnpackUintToUint4(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEBONES + 1]);
	#endif
#endif // GPUSKIN_BONE_INDEX_UINT16

	// Weights
#if GPUSKIN_BONE_WEIGHTS_UINT16
	Out.BlendWeights.xy = UnpackUnorm2x16(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS    ]);
	Out.BlendWeights.zw = UnpackUnorm2x16(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS + 1]);
	#if GPUSKIN_USE_EXTRA_INFLUENCES
	Out.BlendWeights2.xy = UnpackUnorm2x16(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS + 2]);
	Out.BlendWeights2.zw = UnpackUnorm2x16(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS + 3]);
	#endif
#else // GPUSKIN_BONE_WEIGHTS_UINT16
	Out.BlendWeights  = UnpackRGBA8(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS]);
	#if GPUSKIN_USE_EXTRA_INFLUENCES
	Out.BlendWeights2 = UnpackRGBA8(InWeightBuffer[InWeightOffset + GPUSKIN_VB_OFFSET_INFLUENCEWEIGHTS + 1]);
	#endif
#endif // GPUSKIN_BONE_WEIGHTS_UINT16

	return Out;
}

#define FBoneMatrix float3x4

struct FComputeBoneMatrixInputs
{
	FCompressedBoneTransformBuffer BoneMatrices;
	Buffer<uint> SectionBoneMap;
	uint SectionBoneOffset;
	uint BoneTransformOffset;
	uint TransformStorageMode;
};

FComputeBoneMatrixInputs GetComputeBoneMatrixInputs(FCompressedBoneTransformBuffer BoneMatrices)
{
	FComputeBoneMatrixInputs Inputs;
	Inputs.BoneMatrices = BoneMatrices;
	Inputs.SectionBoneOffset   = 0;
	Inputs.BoneTransformOffset = 0;
	Inputs.TransformStorageMode = GPUSKIN_TRANSFORM_STORAGE_BONE_MAP;
	return Inputs;
}

uint GetSkinningBoneTransformOffset(uint TransformBufferOffset, uint InstanceOffset, uint CurrentTransformSlot, uint MaxTransformCount, bool bPreviousTransform)
{
	return TransformBufferOffset + (InstanceOffset + (bPreviousTransform ^ CurrentTransformSlot)) * MaxTransformCount;
}

FBoneMatrix LoadSkinningBoneTransform(FComputeBoneMatrixInputs Inputs, uint TransformIndex)
{
	uint TransformBufferIndex = Inputs.BoneTransformOffset + Inputs.SectionBoneOffset + TransformIndex;

	return LoadCompressedBoneTransformTransposed(Inputs.BoneMatrices, 0, TransformBufferIndex);
}

FBoneMatrix ComputeBoneMatrixWithLimitedInfluences(FComputeBoneMatrixInputs Inputs, FGPUSkinIndexAndWeight In, bool bExtraInfluences = GPUSKIN_USE_EXTRA_INFLUENCES)
{
	FBoneMatrix
	BoneMatrix = In.BlendWeights.x * LoadSkinningBoneTransform(Inputs, In.BlendIndices.x);
	BoneMatrix += In.BlendWeights.y * LoadSkinningBoneTransform(Inputs, In.BlendIndices.y);
#if !GPUSKIN_LIMIT_2BONE_INFLUENCES
	BoneMatrix += In.BlendWeights.z * LoadSkinningBoneTransform(Inputs, In.BlendIndices.z);
	BoneMatrix += In.BlendWeights.w * LoadSkinningBoneTransform(Inputs, In.BlendIndices.w);
	if (bExtraInfluences)
	{
		BoneMatrix += In.BlendWeights2.x * LoadSkinningBoneTransform(Inputs, In.BlendIndices2.x);
		BoneMatrix += In.BlendWeights2.y * LoadSkinningBoneTransform(Inputs, In.BlendIndices2.y);
		BoneMatrix += In.BlendWeights2.z * LoadSkinningBoneTransform(Inputs, In.BlendIndices2.z);
		BoneMatrix += In.BlendWeights2.w * LoadSkinningBoneTransform(Inputs, In.BlendIndices2.w);
	}
#endif // GPUSKIN_LIMIT_2BONE_INFLUENCES
	return BoneMatrix;
}

FBoneMatrix ComputeBoneMatrixWithUnlimitedInfluences(FComputeBoneMatrixInputs Inputs, Buffer<uint> InVertexWeights, uint InWeightIndexSize, uint InBlendOffsetCount)
{
	int NumBoneInfluences = InBlendOffsetCount & 0xff;
	int StreamOffset = InBlendOffsetCount >> 8;
	int BoneIndexSize = InWeightIndexSize & 0xff;
	int BoneWeightSize = InWeightIndexSize >> 8;
	int WeightsOffset = StreamOffset + (BoneIndexSize * NumBoneInfluences);

	FBoneMatrix BoneMatrix = FBoneMatrix(float4(0,0,0,0), float4(0,0,0,0), float4(0,0,0,0));
	for (int InfluenceIdx = 0; InfluenceIdx < NumBoneInfluences; InfluenceIdx++)
	{
		int BoneIndexOffset = StreamOffset + (BoneIndexSize * InfluenceIdx);
		int BoneWeightOffset = WeightsOffset + InfluenceIdx * BoneWeightSize;
		int BoneIndex = InVertexWeights[BoneIndexOffset];
		if (BoneIndexSize > 1)
		{
			BoneIndex = InVertexWeights[BoneIndexOffset + 1] << 8 | BoneIndex;
			//@todo-lh: Workaround to fix issue in SPIRVEmitter of DXC; this block must be inside the if branch
			half BoneWeight;
			if (BoneWeightSize > 1)
			{
				BoneWeight = half( (InVertexWeights[BoneWeightOffset + 1] << 8 | InVertexWeights[BoneWeightOffset]) / 65535.0 );
			}
			else
			{
				BoneWeight = half(InVertexWeights[BoneWeightOffset]) / 255.0f;
			}
			BoneMatrix += BoneWeight * LoadSkinningBoneTransform(Inputs, BoneIndex);
		}
		else
		{
			//@todo-lh: Workaround to fix issue in SPIRVEmitter of DXC; this block must be inside the if branch
			half BoneWeight;
			if (BoneWeightSize > 1)
			{
				BoneWeight = half( (InVertexWeights[BoneWeightOffset + 1] << 8 | InVertexWeights[BoneWeightOffset]) / 65535.0 );
			}
			else
			{
				BoneWeight = half(InVertexWeights[BoneWeightOffset]) / 255.0f;
			}
			BoneMatrix += BoneWeight * LoadSkinningBoneTransform(Inputs, BoneIndex);
		}
	}
	return BoneMatrix;
}

FBoneMatrix ComputeBoneMatrixWithLimitedInfluences(FCompressedBoneTransformBuffer BoneMatrices, FGPUSkinIndexAndWeight In, bool bExtraInfluences = GPUSKIN_USE_EXTRA_INFLUENCES)
{
	FComputeBoneMatrixInputs Inputs = GetComputeBoneMatrixInputs(BoneMatrices);
	return ComputeBoneMatrixWithLimitedInfluences(Inputs, In, bExtraInfluences);
}

FBoneMatrix ComputeBoneMatrixWithUnlimitedInfluences(FCompressedBoneTransformBuffer BoneMatrices, Buffer<uint> InVertexWeights, uint InWeightIndexSize, uint InBlendOffsetCount)
{
	FComputeBoneMatrixInputs Inputs = GetComputeBoneMatrixInputs(BoneMatrices);
	return ComputeBoneMatrixWithUnlimitedInfluences(Inputs, InVertexWeights, InWeightIndexSize, InBlendOffsetCount);
}