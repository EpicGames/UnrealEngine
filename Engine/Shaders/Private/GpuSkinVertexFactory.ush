// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GpuSkinVertexFactory.hlsl: GPU skinned vertex factory shader code
=============================================================================*/

#include "Skinning/SkinningSceneDataDecode.ush"
#include "VertexFactoryCommon.ush"
#include "GpuSkinCommon.ush"

// Whether the material supports morph targets
#ifndef GPUSKIN_MORPH_BLEND
#define GPUSKIN_MORPH_BLEND 0
#endif

#ifndef GPUSKIN_APEX_CLOTH
#define GPUSKIN_APEX_CLOTH 0
#endif

#ifndef GPUSKIN_LIMIT_2BONE_INFLUENCES
#define GPUSKIN_LIMIT_2BONE_INFLUENCES	0 // default is 4 bones 
#endif

#ifndef GPUSKIN_UNLIMITED_BONE_INFLUENCE
#define GPUSKIN_UNLIMITED_BONE_INFLUENCE 0
#endif

#define MAX_INFLUENCES_PER_STREAM	4

#define FIXED_VERTEX_INDEX 0xFFFF

#if GPUSKIN_APEX_CLOTH
	#define GPUSkinVF GPUSkinAPEXClothVF.Common
#else
	#define GPUSkinVF GPUSkinVFBase.Common
#endif

bool IsMorphTarget()
{
#if GPUSKIN_MORPH_BLEND
	return GPUSkinVF.bIsMorphTarget;
#else
	// When Morph targets are not supported on the material, always return false so the associated shader code can be compiled out.
	return false;
#endif
} 

struct FVertexFactoryInput
{
	float4	Position		: ATTRIBUTE0;
	// 0..1
	HALF3_TYPE	TangentX		: ATTRIBUTE1;
	// 0..1
	// TangentZ.w contains sign of tangent basis determinant
	HALF4_TYPE	TangentZ		: ATTRIBUTE2;	

#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
	uint	BlendOffsetCount	: ATTRIBUTE3;
#else
	uint4	BlendIndices	: ATTRIBUTE3;
	uint4	BlendIndicesExtra	: ATTRIBUTE14;
	float4	BlendWeights	: ATTRIBUTE4;	
	float4	BlendWeightsExtra	: ATTRIBUTE15;
#endif // GPUSKIN_UNLIMITED_BONE_INFLUENCE

#if NUM_MATERIAL_TEXCOORDS_VERTEX
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
	float4	TexCoords0 : ATTRIBUTE5;
#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 1
	float2	TexCoords0 : ATTRIBUTE5;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
	float4	TexCoords1 : ATTRIBUTE6;
#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 3
	float2	TexCoords1 : ATTRIBUTE6;
#endif

	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4
		#error Too many texture coordinate sets defined on GPUSkin vertex input. Max: 4.
	#endif
#endif // NUM_MATERIAL_TEXCOORDS_VERTEX

#if GPUSKIN_MORPH_BLEND
	// NOTE: TEXCOORD6,TEXCOORD7 used instead of POSITION1,NORMAL1 since those semantics are not supported by Cg 
	/** added to the Position */
	float3	DeltaPosition	: ATTRIBUTE9;	//POSITION1;
	/** added to the TangentZ and then used to derive new TangentX,TangentY, .w contains the weight of the tangent blend */
	float3	DeltaTangentZ	: ATTRIBUTE10;	//NORMAL1;
#endif

#if GPUSKIN_MORPH_BLEND || GPUSKIN_APEX_CLOTH
	uint VertexID : SV_VertexID;
#endif

	/** Per vertex color */
	float4 Color : ATTRIBUTE13;

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE16
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(16)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

#if GPUSKIN_APEX_CLOTH
// This must match NUM_INFLUENCES_PER_VERTEX in ClothingMeshUtils.cpp and GpuSkinCacheComputeShader.usf
// It represents the maximum number of influences so perhaps should be renamed. Right now we are keeping the name matching for easy search.
// TODO: Make this easier to change in without messing things up
#define NUM_INFLUENCES_PER_VERTEX 5

struct FClothVertex
{
    // APEX cloth mesh-mesh mapping data
    // Barycentric Coordinate Data
    float4	BaryCoordPos;
	float4	BaryCoordNormal;
	float4	BaryCoordTangent;
	uint4	SimulIndices;

    float Weight;

/*
	// Contains the 3 indices for verts in the source mesh forming a triangle, the last element
	// is a flag to decide how the skinning works, 0xffff uses no simulation, and just normal
	// skinning, anything else uses the source mesh and the above skin data to get the final position
	uint16	 SourceMeshVertIndices[4];

	// Dummy for alignment (16 bytes)
	uint32	 Padding[2];
*/
};


#define NUM_FLOAT4S_PER_VERTEX_INFLUENCE 4;   // stride of GPUSkinApexCloth

void SetupClothVertex(uint VertexIndex, out FClothVertex ClothVertexInfluences[NUM_INFLUENCES_PER_VERTEX])
{
	const uint MinVertexIndex = GPUSkinAPEXClothVF.GPUSkinApexClothStartIndexOffset.x;
	const uint ClothIndexOffset = GPUSkinAPEXClothVF.GPUSkinApexClothStartIndexOffset.y;
    VertexIndex = GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex * (VertexIndex - MinVertexIndex) + ClothIndexOffset;

    UNROLL_N(NUM_INFLUENCES_PER_VERTEX)
	for (int i = 0; i < GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex; ++i )
    {
        const uint Offset = (VertexIndex + i) * NUM_FLOAT4S_PER_VERTEX_INFLUENCE;

        ClothVertexInfluences[i].BaryCoordPos = GPUSkinAPEXClothVF.GPUSkinApexCloth[Offset];
        ClothVertexInfluences[i].BaryCoordNormal = GPUSkinAPEXClothVF.GPUSkinApexCloth[Offset + 1];
        ClothVertexInfluences[i].BaryCoordTangent = GPUSkinAPEXClothVF.GPUSkinApexCloth[Offset + 2];

	    uint4 PackedIndices = asuint(GPUSkinAPEXClothVF.GPUSkinApexCloth[Offset + 3]);
        ClothVertexInfluences[i].SimulIndices.yw = (PackedIndices.xy >> 16) & 0xffff;
        ClothVertexInfluences[i].SimulIndices.xz = PackedIndices.xy & 0xffff;

        ClothVertexInfluences[i].Weight = asfloat(PackedIndices[2]);
    }
}
#endif

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if INTERPOLATE_VERTEX_COLOR
	float4 Color : COLOR0;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	// Pack interpolators to reduce the number of semantics used
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2]	: TEXCOORD0;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	// Note: SHADER_USES_PRIMITIVE_UBO pushes the DrawInstanceId through InstanceId.
	nointerpolation uint InstanceId : INSTANCE_ID;
#endif
};

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS;CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif
	Result.UnMirrored = TangentToWorld2.w;
#if INTERPOLATE_VERTEX_COLOR
	Result.VertexColor = Interpolants.Color;
#else
	Result.VertexColor = 0;
#endif
	Result.TwoSidedSign = 1;

#if VF_USE_PRIMITIVE_SCENE_DATA
	FInstanceSceneData InstanceData = GetInstanceSceneData(Interpolants.InstanceId);
	Result.PrimitiveId = InstanceData.PrimitiveId;

	#if USES_PER_INSTANCE_CUSTOM_DATA
		Result.CustomDataOffset = InstanceData.CustomDataOffset;
		Result.CustomDataCount  = InstanceData.CustomDataCount;
	#endif
		Result.PerInstanceRandom = InstanceData.RandomID;

	#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
		Result.InstanceLocalToWorld = InstanceData.LocalToWorld;
	#endif
	#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
		Result.InstanceWorldToLocal = InstanceData.WorldToLocal;
	#endif
#else
	Result.PrimitiveId = 0;
#endif

	return Result;
}

#define FBoneMatrix float3x4

// Cache data to avoid multiple calculation 
struct FVertexFactoryIntermediates
{
	float3 InvNonUniformScale;
	float DeterminantSign;

	FDFMatrix LocalToWorld;
	FDFInverseMatrix WorldToLocal;
	FDFMatrix PrevLocalToWorld;

	// Blend Matrix (used for position/tangents)
	FBoneMatrix BlendMatrix;

	// Unpacked position
	float3 UnpackedPosition;
	
	// Tangent Basis
	float3x3 TangentToLocal; 

	// Vertex Color
	float4 Color;

#if GPUSKIN_APEX_CLOTH
	// in world space (non translated)
	float4 SimulatedPosition;
	// in world space (non translated)
	float4 PreviousSimulatedPosition;

	FClothVertex ClothVertexInfluences[NUM_INFLUENCES_PER_VERTEX];
#endif
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;

	/** Whether World Position Offset should be evaluated for this draw (gated by WPO disable distance). */
	bool bEvaluateWorldPositionOffset;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

FInstanceSceneData GetInstanceData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.InstanceData;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	Result.PrimitiveId = Intermediates.SceneData.PrimitiveId;

	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.Color;

	float3x3 LocalToWorldNoScale = DFToFloat3x3(Intermediates.LocalToWorld);
	LocalToWorldNoScale[0] *= Intermediates.InvNonUniformScale.x;
	LocalToWorldNoScale[1] *= Intermediates.InvNonUniformScale.y;
	LocalToWorldNoScale[2] *= Intermediates.InvNonUniformScale.z;
	Result.TangentToWorld = mul(TangentToLocal, LocalToWorldNoScale);

	Result.PreSkinnedPosition = Intermediates.UnpackedPosition.xyz;
	Result.PreSkinnedNormal = Input.TangentZ.xyz;
#if USE_INSTANCE_CULLING
	Result.InstanceLocalToWorld = Intermediates.SceneData.InstanceData.LocalToWorld;
	Result.InstanceWorldToLocal = Intermediates.SceneData.InstanceData.WorldToLocal;
#endif
#if USES_PER_INSTANCE_CUSTOM_DATA && VF_USE_PRIMITIVE_SCENE_DATA
	Result.CustomDataOffset = Intermediates.SceneData.InstanceData.CustomDataOffset;
	Result.CustomDataCount  = Intermediates.SceneData.InstanceData.CustomDataCount;
#endif
	Result.PerInstanceRandom = Intermediates.SceneData.InstanceData.RandomID;
	Result.PrevFrameLocalToWorld = Intermediates.PrevLocalToWorld;
	Result.bEvaluateWorldPositionOffset = Intermediates.bEvaluateWorldPositionOffset;

#if NUM_MATERIAL_TEXCOORDS_VERTEX

#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
	Result.TexCoords[0] = Input.TexCoords0.xy;
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
	Result.TexCoords[1] = Input.TexCoords0.zw;
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
	Result.TexCoords[2] = Input.TexCoords1.xy;
#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
	Result.TexCoords[3] = Input.TexCoords1.zw;
#endif

#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

#if GPUSKIN_APEX_CLOTH

// if false, fixed vertex
bool IsSimulatedVertex(FClothVertex Input)
{
	return (Input.SimulIndices.w < FIXED_VERTEX_INDEX);
}

float GetClothSimulWeight(uint StaticAlpha)
{
	return 1.0f - (StaticAlpha / 65535.0f);
}

float3 GetClothSimulPosition(int Index, bool bPrevious)
{
	float3 Position;
	if(bPrevious)
	{
		return float3(GPUSkinAPEXClothVF.PreviousClothSimulVertsPositionsNormals[Index * 3], GPUSkinAPEXClothVF.PreviousClothSimulVertsPositionsNormals[Index * 3 + 1].x);
	}
	else
	{
		return float3(GPUSkinAPEXClothVF.ClothSimulVertsPositionsNormals[Index * 3], GPUSkinAPEXClothVF.ClothSimulVertsPositionsNormals[Index * 3 + 1].x);
	}
}

float3 GetClothSimulNormal(int Index, bool bPrevious)
{
	float3 Normal;
	if(bPrevious)
	{
		return float3(GPUSkinAPEXClothVF.PreviousClothSimulVertsPositionsNormals[Index * 3 + 1].y, GPUSkinAPEXClothVF.PreviousClothSimulVertsPositionsNormals[Index * 3 + 2]);
	}
	else
	{
		return float3(GPUSkinAPEXClothVF.ClothSimulVertsPositionsNormals[Index * 3 + 1].y, GPUSkinAPEXClothVF.ClothSimulVertsPositionsNormals[Index * 3 + 2]);
	}
}

float4 ClothingPosition(FClothVertex ClothVertexInfluences[NUM_INFLUENCES_PER_VERTEX], bool bPrevious)
{
	int NumInfluences = 0;
	float SumWeights = 0.0f;
	float SimulWeight = 0.0f; 
	float3 AveragedSimulatedPosition = float3(0, 0, 0);

	const float3 WorldScaleAbs = abs(GPUSkinAPEXClothVF.WorldScale);  // World scale can't be used mirrored to calculate the clothing positions and tangents since the cloth normals are then reversed

	UNROLL_N(NUM_INFLUENCES_PER_VERTEX)
    for (int i = 0; i < GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex; ++i )
    {
        const FClothVertex Influence = ClothVertexInfluences[i];

	    if( IsSimulatedVertex(Influence) )
	    { 
            ++NumInfluences;

            float3 A = GetClothSimulPosition(Influence.SimulIndices.x, bPrevious);
		    float3 B = GetClothSimulPosition(Influence.SimulIndices.y, bPrevious);
		    float3 C = GetClothSimulPosition(Influence.SimulIndices.z, bPrevious);

		    float3 NA = GetClothSimulNormal(Influence.SimulIndices.x, bPrevious);
		    float3 NB = GetClothSimulNormal(Influence.SimulIndices.y, bPrevious);
		    float3 NC = GetClothSimulNormal(Influence.SimulIndices.z, bPrevious);

	    	SimulWeight += GetClothSimulWeight(Influence.SimulIndices.w);
	    	
			float Weight = 1.0f;
			if ( GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex > 1 )
			{
				Weight = Influence.Weight;
				SumWeights += Weight;
			}
			else
			{
				// Single influence, weight is 1.0
	            Weight = 1.0f;
                SumWeights = 1.0f;
			}

			float3 TriangleBary = float3(Influence.BaryCoordPos.x,
                                         Influence.BaryCoordPos.y,
										 1.0f - Influence.BaryCoordPos.x - Influence.BaryCoordPos.y);

			float3 SimPosition = TriangleBary.x * (A + NA * Influence.BaryCoordPos.w * WorldScaleAbs.x)
					           + TriangleBary.y * (B + NB * Influence.BaryCoordPos.w * WorldScaleAbs.y)
					           + TriangleBary.z * (C + NC * Influence.BaryCoordPos.w * WorldScaleAbs.z);

			AveragedSimulatedPosition += Weight * SimPosition;
	    }
    }

    if ( NumInfluences > 0 && SumWeights > 1e-4f )
    {
		float InvWeight = 1.0f / SumWeights; 
		AveragedSimulatedPosition *= InvWeight;
	}
	else
	{
		AveragedSimulatedPosition = float3(0, 0, 0);
		SimulWeight = 0.0f;			// Fall back to skinned vertex position by setting this lerp param to zero
	}

	if (GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex > 1)
	{
		SimulWeight /= GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex;
	}

	return float4(AveragedSimulatedPosition, SimulWeight);
}

#endif //#if GPUSKIN_APEX_CLOTH

/**
* Unpack position - uncompress xyz position to world position
*/
float3 UnpackedPosition( FVertexFactoryInput Input )
{
		return float3(Input.Position.xyz);
}

#if GPUSKIN_MORPH_BLEND

/** 
* Adds the delta position from the combined morph targets to the vertex position 
*/
float3 MorphPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.UnpackedPosition + Input.DeltaPosition;
}

float3 MorphPreviousPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	if (ResolvedView.FrameCounter == GPUSkinVF.MorphUpdatedFrameNumber && ResolvedView.WorldIsPaused == 0)
	{
		// 3 + 3 floats because of the format defined in FMorphGPUSkinVertex
		uint Offset = Input.VertexID * (3 + 3);

		float3 PreviousDeltaPosition = float3(
			GPUSkinVF.PreviousMorphBuffer[Offset + 0],
			GPUSkinVF.PreviousMorphBuffer[Offset + 1],
			GPUSkinVF.PreviousMorphBuffer[Offset + 2]);

		return Intermediates.UnpackedPosition + PreviousDeltaPosition;
	}
	return Intermediates.UnpackedPosition + Input.DeltaPosition;
}
#endif // GPUSKIN_MORPH_BLEND

FComputeBoneMatrixInputs GetComputeBoneMatrixInputs(FInstanceSceneData SceneData, FCompressedBoneTransformBuffer BoneMatrices, bool bPreviousTransform)
{
	FComputeBoneMatrixInputs Inputs;
	Inputs.BoneMatrices = BoneMatrices;
	Inputs.SectionBoneMap = GPUSkinVF.SectionBoneMap;
	Inputs.BoneTransformOffset = GetSkinningBoneTransformOffset(GPUSkinVF.TransformBufferOffset, SceneData.SkinningData * (GPUSkinVF.bIsRefPose ? 0 : 1), GPUSkinVF.CurrentTransformSlot, GPUSkinVF.MaxTransformCount, bPreviousTransform);
	Inputs.TransformStorageMode = GPUSkinVF.TransformStorageMode;
	Inputs.SectionBoneOffset = GPUSkinVF.SectionBoneOffset;
	return Inputs;
}

FBoneMatrix CalcBoneMatrix(FVertexFactoryInput Input, FInstanceSceneData SceneData)
{
	FComputeBoneMatrixInputs BoneMatrixInputs = GetComputeBoneMatrixInputs(SceneData, GPUSkinVF.BoneMatrices, false);

#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
	return ComputeBoneMatrixWithUnlimitedInfluences(BoneMatrixInputs, GPUSkinVF.InputWeightStream, GPUSkinVF.InputWeightIndexSize, Input.BlendOffsetCount);
#else
	const bool bHasExtraInfluences = GPUSkinVF.NumBoneInfluencesParam > MAX_INFLUENCES_PER_STREAM;
	FGPUSkinIndexAndWeight In = (FGPUSkinIndexAndWeight)0;
	In.BlendIndices = Input.BlendIndices;
	In.BlendWeights = Input.BlendWeights;
	if (bHasExtraInfluences)
	{
		In.BlendIndices2 = Input.BlendIndicesExtra;
		In.BlendWeights2 = Input.BlendWeightsExtra;
	}
	return ComputeBoneMatrixWithLimitedInfluences(BoneMatrixInputs, In, bHasExtraInfluences);
#endif
}

FBoneMatrix CalcPreviousBoneMatrix(FVertexFactoryInput Input, FInstanceSceneData SceneData)
{
	FComputeBoneMatrixInputs BoneMatrixInputs = GetComputeBoneMatrixInputs(SceneData, GPUSkinVF.PreviousBoneMatrices, true);

#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
	return ComputeBoneMatrixWithUnlimitedInfluences(BoneMatrixInputs,  GPUSkinVF.InputWeightStream, GPUSkinVF.InputWeightIndexSize, Input.BlendOffsetCount);
#else
	const bool bHasExtraInfluences = GPUSkinVF.NumBoneInfluencesParam > MAX_INFLUENCES_PER_STREAM;
	FGPUSkinIndexAndWeight In = (FGPUSkinIndexAndWeight)0;
	In.BlendIndices = Input.BlendIndices;
	In.BlendWeights = Input.BlendWeights;
	if (bHasExtraInfluences)
	{
		In.BlendIndices2 = Input.BlendIndicesExtra;
		In.BlendWeights2 = Input.BlendWeightsExtra;
	}
	return ComputeBoneMatrixWithLimitedInfluences(BoneMatrixInputs, In, bHasExtraInfluences);
#endif
}

/** transform position by weighted sum of skinning matrices */
float3 SkinPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3 Position;
	if (IsMorphTarget())
	{
	#if GPUSKIN_MORPH_BLEND
		Position = MorphPosition(Input, Intermediates);
	#endif
	}
	else
	{
		Position = Intermediates.UnpackedPosition;
	}

	// Note the use of mul(Matrix,Vector), bone matrices are stored transposed
	// for tighter packing.
	Position = mul(Intermediates.BlendMatrix, float4(Position, 1));

	return Position;
}

/** transform position by weighted sum of skinning matrices */
float3 SkinPreviousPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3 Position;
	if (IsMorphTarget())
	{
	#if GPUSKIN_MORPH_BLEND
		Position = MorphPreviousPosition(Input, Intermediates);
	#endif
	}
	else
	{
		Position = Intermediates.UnpackedPosition;
	}

	FBoneMatrix BlendMatrix = Intermediates.BlendMatrix;

	// Previous Blend Matrix (used for position in velocity rendering)
	if (GPUSkinVF.BoneUpdatedFrameNumber == ~0u || (ResolvedView.FrameCounter == GPUSkinVF.BoneUpdatedFrameNumber && ResolvedView.WorldIsPaused == 0))
	{
#if defined(IS_VELOCITY_SHADER) && STEREO_MOTION_VECTORS
		FBoneMatrix PrevBlendMatrix = CalcPreviousBoneMatrix(Input, Intermediates.SceneData.InstanceData);
		if (IsSmoothRotation(PrevBlendMatrix._11_21_31, PrevBlendMatrix._12_22_32, BlendMatrix._11_21_31, BlendMatrix._12_22_32))
			BlendMatrix = PrevBlendMatrix;
#else
		BlendMatrix = CalcPreviousBoneMatrix(Input, Intermediates.SceneData.InstanceData);
#endif
	}

	// Note the use of mul(Matrix,Vector), bone matrices are stored transposed
	// for tighter packing.
	Position = mul(BlendMatrix, float4(Position, 1));

	return Position;
}
/** transform the tangent basis vectors */
float3x3 SkinTangents( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3x3 TangentToLocal;

	// tangent
	// -1..1
	half3 LocalTangentX = Input.TangentX;
	// -1..1 .xyz:normal, .w:contains sign of tangent basis determinant (left or right handed)
	half4 LocalTangentZ = Input.TangentZ;

#if GPUSKIN_APEX_CLOTH
	float TempClothBlendWeight = 0.0f;
	float3x3 ClothTangent = float3x3(1,0,0, 0,1,0, 0,0,1);
	if (GPUSkinAPEXClothVF.bEnabled && IsSimulatedVertex(Intermediates.ClothVertexInfluences[0]))
	{ 
		TempClothBlendWeight = GPUSkinAPEXClothVF.ClothBlendWeight;
		bool bPrevious = false;

		float3 NormalPosition = float3(0,0,0);
		float3 TangentPosition = float3(0,0,0);
		int NumInfluences = 0;
		float SumWeights = 0.0;
		float SimulWeight = 0.0;

		const float3 WorldScaleAbs = abs(GPUSkinAPEXClothVF.WorldScale);  // World scale can't be used mirrored to calculate the clothing positions and tangents since the cloth normals are then reversed

		for (int i = 0; i < GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex; ++i )
		{
			const FClothVertex Influence = Intermediates.ClothVertexInfluences[i];

			if( IsSimulatedVertex(Influence) )
			{ 
				++NumInfluences;
				TempClothBlendWeight = GPUSkinAPEXClothVF.ClothBlendWeight;

				float3 A = GetClothSimulPosition(Influence.SimulIndices.x, bPrevious);
				float3 B = GetClothSimulPosition(Influence.SimulIndices.y, bPrevious);
				float3 C = GetClothSimulPosition(Influence.SimulIndices.z, bPrevious);

				float3 NA = GetClothSimulNormal(Influence.SimulIndices.x, bPrevious);
				float3 NB = GetClothSimulNormal(Influence.SimulIndices.y, bPrevious);
				float3 NC = GetClothSimulNormal(Influence.SimulIndices.z, bPrevious);

	        	SimulWeight += GetClothSimulWeight(Influence.SimulIndices.w);
	        	
				float Weight = 1.0f;
				if ( GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex > 1 )
				{
					// Weight is packed in the last coordinate
					Weight = Influence.Weight;
					SumWeights += Weight;
				}
				else
				{
					// Single influence, weight is 1.0
					Weight = 1.0f;
					SumWeights = 1.0f;	   
				}

				NormalPosition += Weight * (Influence.BaryCoordNormal.x * (A + NA * Influence.BaryCoordNormal.w * WorldScaleAbs.x)
											+ Influence.BaryCoordNormal.y * (B + NB * Influence.BaryCoordNormal.w * WorldScaleAbs.y)
											+ Influence.BaryCoordNormal.z * (C + NC * Influence.BaryCoordNormal.w * WorldScaleAbs.z));

				TangentPosition += Weight * (Influence.BaryCoordTangent.x * (A + NA * Influence.BaryCoordTangent.w * WorldScaleAbs.x)
											+ Influence.BaryCoordTangent.y * (B + NB * Influence.BaryCoordTangent.w * WorldScaleAbs.y)
											+ Influence.BaryCoordTangent.z * (C + NC * Influence.BaryCoordTangent.w * WorldScaleAbs.z));
			}
		}

		if (GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex > 1)
		{
			SimulWeight /= GPUSkinAPEXClothVF.ClothNumInfluencesPerVertex;
		}
	
		if ( NumInfluences > 0 && SumWeights > 1e-4f )
		{
			float InvWeight = 1.0f / SumWeights; 
			TangentPosition *= InvWeight;
			NormalPosition *= InvWeight;

			TangentToLocal[0] = normalize(TangentPosition - Intermediates.SimulatedPosition.xyz);
			TangentToLocal[2] = normalize(NormalPosition - Intermediates.SimulatedPosition.xyz);

			// Simulated cloth data are in cloth space so need to change into local space
			TangentToLocal[0] = mul(float4(TangentToLocal[0], 0.0f), GPUSkinAPEXClothVF.ClothToLocal).xyz;
			TangentToLocal[2] = mul(float4(TangentToLocal[2], 0.0f), GPUSkinAPEXClothVF.ClothToLocal).xyz;
	
			// derive the new binormal by getting the cross product of the normal and tangent
			// and flip vector based on sign of tangent basis determinant
			TangentToLocal[1] = cross(TangentToLocal[2], TangentToLocal[0]) * LocalTangentZ.w;
	
			ClothTangent = TangentToLocal;
		}
		else
		{
			SimulWeight = 0.0f;			// Fall back to skinned data by setting this lerp param to zero
		}

		TempClothBlendWeight *= SimulWeight;
	}
#else
	if (IsMorphTarget())
	{
#if GPUSKIN_MORPH_BLEND
		// calc new normal by offseting it with the delta
		LocalTangentZ.xyz = normalize(LocalTangentZ.xyz + Input.DeltaTangentZ);
		// derive the new tangent by orthonormalizing the new normal against
		// the base tangent vector (assuming these are normalized)
		LocalTangentX = normalize(LocalTangentX - dot(LocalTangentX, LocalTangentZ.xyz) * LocalTangentZ.xyz);
#endif
	}
#endif // GPUSKIN_APEX_CLOTH


	// Note the use of mul(Matrix,Vector), bone matrices are stored transposed
	// for tighter packing.
	TangentToLocal[0] = normalize(mul( Intermediates.BlendMatrix, float4(LocalTangentX, 0) ));
	TangentToLocal[2] = normalize(mul( Intermediates.BlendMatrix, float4(LocalTangentZ.xyz, 0) ));

	// derive the new binormal by getting the cross product of the normal and tangent
	// and flip vector based on sign of tangent basis determinant
	 TangentToLocal[1] = normalize(cross(TangentToLocal[2], TangentToLocal[0]) * LocalTangentZ.w);

#if GPUSKIN_APEX_CLOTH
	if (GPUSkinAPEXClothVF.bEnabled)
	{
		return ClothTangent * TempClothBlendWeight + TangentToLocal * (1.0f - TempClothBlendWeight);
	}
#endif	// GPUSKIN_APEX_CLOTH
	return TangentToLocal;
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData				= VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.InvNonUniformScale	= GetInstanceData(Intermediates).InvNonUniformScale;
	Intermediates.DeterminantSign		= GetInstanceData(Intermediates).DeterminantSign;
	Intermediates.LocalToWorld			= GetInstanceData(Intermediates).LocalToWorld;
	Intermediates.WorldToLocal			= GetInstanceData(Intermediates).WorldToLocal;
	Intermediates.PrevLocalToWorld		= GetInstanceData(Intermediates).PrevLocalToWorld;
	Intermediates.UnpackedPosition		= UnpackedPosition(Input);

	// Evaluate WPO unless the primitive opted out or if the camera is past the WPO disable distance.
	Intermediates.bEvaluateWorldPositionOffset = true;
#if USE_INSTANCE_CULLING
	Intermediates.bEvaluateWorldPositionOffset = (Intermediates.SceneData.CullingFlags & INSTANCE_CULLING_FLAG_EVALUATE_WPO) != 0;
#elif USES_WORLD_POSITION_OFFSET && !ALWAYS_EVALUATE_WORLD_POSITION_OFFSET && (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
	{
		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Intermediates);
		const bool bPrimEvalWPO = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET) != 0;
		const bool bWPODisableDistance = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE) != 0;

		// View distance computed from primitive bounds center
		const float InstanceViewDistSq = length2(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation));

		if (!bPrimEvalWPO || (bWPODisableDistance && InstanceViewDistSq >= PrimitiveData.InstanceWPODisableDistanceSquared))
		{
			Intermediates.bEvaluateWorldPositionOffset = false;
		}
	}
#endif

#if GPUSKIN_APEX_CLOTH
	if (GPUSkinAPEXClothVF.bEnabled)
	{
		SetupClothVertex(Input.VertexID, Intermediates.ClothVertexInfluences);
		if( IsSimulatedVertex(Intermediates.ClothVertexInfluences[0]) )
		{
			Intermediates.SimulatedPosition = ClothingPosition(Intermediates.ClothVertexInfluences, false);
			Intermediates.PreviousSimulatedPosition = ClothingPosition(Intermediates.ClothVertexInfluences, true);
		}
		else
		{
			Intermediates.PreviousSimulatedPosition = Intermediates.SimulatedPosition = float4(Intermediates.UnpackedPosition, 0.0f);
		}
	}
#endif

	Intermediates.BlendMatrix = CalcBoneMatrix(Input, Intermediates.SceneData.InstanceData);

	// Fill TangentToLocal
	Intermediates.TangentToLocal = SkinTangents(Input, Intermediates);

	// Swizzle vertex color.
	Intermediates.Color = Input.Color FCOLOR_COMPONENT_SWIZZLE;

	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToLocal;
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float3 SkinnedPosition = SkinPosition(Input, Intermediates);
	#if GPUSKIN_APEX_CLOTH
	if (GPUSkinAPEXClothVF.bEnabled && IsSimulatedVertex(Intermediates.ClothVertexInfluences[0]))
	{
		// Transform simulated position from cloth space into local space and blend with skinned position
		float4 TransformedSimulatedPos = mul(float4(Intermediates.SimulatedPosition.xyz, 1), GPUSkinAPEXClothVF.ClothToLocal);
		SkinnedPosition = lerp(SkinnedPosition, TransformedSimulatedPos.xyz, GPUSkinAPEXClothVF.ClothBlendWeight * Intermediates.SimulatedPosition.w);
	}
	#endif
	return SkinnedPosition;
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformLocalToTranslatedWorld(VertexFactoryGetInstanceSpacePosition(Input, Intermediates), Intermediates.LocalToWorld);
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

void CalcTangentToWorld(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, out float3 TangentToWorld0, out float4 TangentToWorld2)
{
	float3x3 LocalToWorld = DFToFloat3x3(Intermediates.LocalToWorld);

	// Remove scaling.
	half3 InvScale = Intermediates.InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;

	float3x3 TangentToWorld = mul(Intermediates.TangentToLocal, LocalToWorld);

	TangentToWorld0 = TangentToWorld[0];
	TangentToWorld2 = float4(TangentToWorld[2], Input.TangentZ.w * Intermediates.DeterminantSign);
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float3 TangentToWorld0; 
	float4 TangentToWorld2;
	CalcTangentToWorld(Input, Intermediates, TangentToWorld0, TangentToWorld2);

	return TangentToWorld2.xyz;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;	// compiling the SetUV loop will fail if not completely initialized

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS]; // = (float2[NUM_TEX_COORD_INTERPOLATORS])0; <- our FHlslParser used for RemoveUnusedOutputs doesn't seem to like this, so zero-initialize manually below.
	// Why zero-initialize? Because material translator stores interpolants in an array of float2, packing them, but the material may have an odd number of interpolants (e.g. float3).
	// In such a case GetCustomInterpolators() will omit writing the last component, leaving it uninitialized, hence the initialization below.
	CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS - 1] = float2(0, 0);	

	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

	Interpolants.TangentToWorld0.w = 0;
	CalcTangentToWorld(Input, Intermediates, Interpolants.TangentToWorld0.xyz, Interpolants.TangentToWorld2);
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Interpolants.TangentToWorld2_Center = Interpolants.TangentToWorld2;
#endif

#if INTERPOLATE_VERTEX_COLOR
	Interpolants.Color = Intermediates.Color;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
#if SHADER_USES_PRIMITIVE_UBO
	// Note: SHADER_USES_PRIMITIVE_UBO pushes the DrawInstanceId through SceneData.PrimitiveId
	Interpolants.InstanceId = Intermediates.SceneData.PrimitiveId;
#else
	Interpolants.InstanceId = Intermediates.SceneData.InstanceId;
#endif
#endif

	return Interpolants;
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float3 PrevSkinnedPosition = SkinPreviousPosition(Input, Intermediates);
#if GPUSKIN_APEX_CLOTH
	if (GPUSkinAPEXClothVF.bEnabled && IsSimulatedVertex(Intermediates.ClothVertexInfluences[0]))
	{
		// Transform simulated position from cloth space into local space and blend with skinned position
		float4 PrevTransformedSimulatedPos = mul(float4(Intermediates.PreviousSimulatedPosition.xyz, 1), GPUSkinAPEXClothVF.PreviousClothToLocal);
		PrevSkinnedPosition = lerp(PrevSkinnedPosition, PrevTransformedSimulatedPos.xyz, GPUSkinAPEXClothVF.ClothBlendWeight * Intermediates.PreviousSimulatedPosition.w);
	}
#endif // GPUSKIN_APEX_CLOTH
	return PrevSkinnedPosition;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{	
	float3 PrevSkinnedPosition = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	return TransformPreviousLocalPositionToTranslatedWorld(PrevSkinnedPosition, Intermediates.PrevLocalToWorld);
}


#if NEEDS_VERTEX_FACTORY_INTERPOLATION
float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0,0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	
	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	// Default initialize. Otherwise, some graphics pipelines that
	// couple tessellation with geometry shaders won't write to all TEXCOORD semantics,
	// but read from them when <FVertexFactoryRayTracingInterpolants> is being copied as a whole.
	FVertexFactoryRayTracingInterpolants O = (FVertexFactoryRayTracingInterpolants)0;

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	// TexCoords in VSToPS are a float4 array with 2 coords packed together
	for(int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS+1)/2; ++tc)
	{
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
	}
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.InstanceId = a.InterpolantsVSToPS.InstanceId;
#endif

	return O;
}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
#if SHADER_USES_PRIMITIVE_UBO
	// Note: SHADER_USES_PRIMITIVE_UBO pushes the DrawInstanceId through InstanceId
	return Interpolants.InstanceId;
#else
	FInstanceSceneData InstanceData = GetInstanceSceneData(Interpolants.InstanceId);
	return InstanceData.PrimitiveId;
#endif
#else // VF_USE_PRIMITIVE_SCENE_DATA
	return 0;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
}

#include "VertexFactoryDefaultInterface.ush"
