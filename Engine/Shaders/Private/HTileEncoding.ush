// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

#ifdef OVERRIDE_HTILELOOKUP_USH

	#include "/Platform/Private/HTileLookup.ush"

#else

	uint ComputeTileOffset(uint2 PixelPos, uint PixelsWide, uint PlatformConfig)
	{
		return 0;
	}

#endif

#ifndef PLATFORM_SUPPORTS_HTILE_LOOKUP
#define PLATFORM_SUPPORTS_HTILE_LOOKUP 0
#endif

#if PLATFORM_SUPPORTS_HTILE_LOOKUP

// Optimized version for no hi-stencil, just min and max depth.
uint EncodeTileMinMaxDepth(float MinDepth, float MaxDepth)
{
	// Convert min and max depth to UNORM14.
#if COMPILER_SUPPORTS_PACK_INTRINSICS
	const uint HTileValue = PackFloat32ToUNorm16(MinDepth - 0.5 / 65535.0,
												 MaxDepth + 3.5 / 65535.0);
#else
	const float MaxDepthRange = float((1 << 14) - 1);
	const uint ZMin = uint(floor(MinDepth * MaxDepthRange));
	const uint ZMax = uint( ceil(MaxDepth * MaxDepthRange));
	const uint HTileValue = (ZMin << 18) | (ZMax << 4);
#endif

	// Shift up min by 2 bits, then set all four low bits.
	return BitFieldInsertU32(BitFieldMaskU32(14, 18), HTileValue, HTileValue << 2) | HTILE_ZMASK_EXPANDED;
}

uint EncodeZRangeDelta(uint ZBase, uint ZOther)
{
	uint Delta = abs(int(ZBase) - int(ZOther));
	if (Delta < 16u)
	{
		return Delta;
	}

	const uint LeadingZeros = (31u - firstbithigh(Delta)) - 18u;
	if (LeadingZeros >= 8)
	{
		return ((11u - LeadingZeros) << 3u) | ((Delta >> (10u - LeadingZeros)) & 7u);
	}
	else
	{
		return ((15u - LeadingZeros) << 2u) | ((Delta >> (11u - LeadingZeros)) & 3u);
	}
}

uint EncodeTileStencilZBaseDelta(float MinDepth, float MaxDepth, uint XX, uint SMem, uint SR0, uint SR1, bool bZCompareGE)
{
	const float MaxDepthRange = float((1 << 14) - 1);
	
	const uint ZMask = HTILE_ZMASK_EXPANDED;
	const uint ZMin  = uint(floor(MinDepth * MaxDepthRange));
	const uint ZMax  = uint( ceil(MaxDepth * MaxDepthRange));
	
	// Assumes reverse Z
	const uint ZBase = select(bZCompareGE, ZMax, ZMin);
	const uint ZDelta = EncodeZRangeDelta(ZMin, ZMax);
	
	uint HTileValue = 0;
	HTileValue |= ZMask;				//  0:3
	HTileValue |= (SR0  << 4);			//  4:5
	HTileValue |= (SR1  << 6);			//  6:7
	HTileValue |= (SMem << 8);			//  8:9
	HTileValue |= (XX << 10);			// 10:11
	HTileValue |= (ZDelta << 12);		// 12:17
	HTileValue |= (ZBase << 18);		// 18:31
	return HTileValue;
}

// Decodes 6 bit depth delta coding for hi-stencil.
uint DecodeDepthDelta(uint DeltaZ)
{
	// 000DDD -> 00000000000DDD
	// 001DDD -> 00000000001DDD
	// 010DDD -> 0000000001DDD1
	// 011DDD -> 000000001DDD11
	// 1000DD -> 00000001DD1111
	// 1001DD -> 0000001DD11111
	// 1010DD -> 000001DD111111
	// 1011DD -> 00001DD1111111
	// 1100DD -> 0001DD11111111
	// 1101DD -> 001DD111111111
	// 1110DD -> 01DD1111111111
	// 1111DD -> 1DD11111111111

	const bool TwoBitDelta = BitFieldExtractU32(DeltaZ, 1, 5);
	const uint DeltaBits   = TwoBitDelta ? 2 : 3;
	const uint Delta       = BitFieldExtractU32(DeltaZ, DeltaBits, 0);
	const uint Code        = BitFieldExtractU32(DeltaZ, 3, DeltaBits);

	const uint LeadingOne = Code + (TwoBitDelta ? 6 : 2);
	const uint Ones = (1u << (LeadingOne + 1)) - 1;
	const uint DeltaStart = (LeadingOne >= DeltaBits) ? (LeadingOne - DeltaBits) : 0;

	const uint Mask = BitFieldMaskU32(DeltaBits, DeltaStart);
	return BitFieldInsertU32(Mask, Delta << DeltaStart, Ones);
}

// Decodes the hi-z information in the tile as a 14 bit integer min and max.
uint2 DecodeTileMinMax(uint HTileValue, bool HiStencil, bool CompareMinZ)
{
	uint MinZ;
	uint MaxZ;

	if (HiStencil)
	{
		const uint ZBase  = BitFieldExtractU32(HTileValue, 14, 18); // 14 bit fixed point.
		const uint ZDelta = BitFieldExtractU32(HTileValue,  6, 12); //  6 bit delta coding.

		// Base is closest to near plane; delta is towards far plane.
		MinZ = CompareMinZ ? (ZBase - DecodeDepthDelta(ZDelta)) : ZBase;
		MaxZ = CompareMinZ ?  ZBase : (ZBase + DecodeDepthDelta(ZDelta));
	}
	else
	{
		// Both values are 14 bit fixed point.
		MinZ = BitFieldExtractU32(HTileValue, 14,  4);
		MaxZ = BitFieldExtractU32(HTileValue, 14, 18);
	}

	return uint2(MinZ, MaxZ);
}

// Decodes the hi-stencil information in the tile as a pair of 2 bit values.
uint2 ExtractTileHiStencil(uint HTileValue, bool HiStencil)
{
	uint HiStencil0;
	uint HiStencil1;

	if (HiStencil)
	{
		HiStencil0 = BitFieldExtractU32(HTileValue, 2, 4);
		HiStencil1 = BitFieldExtractU32(HTileValue, 2, 6);
	}
	else
	{
		HiStencil0 = 0;
		HiStencil1 = 0;
	}

	return uint2(HiStencil0, HiStencil1);
}

uint GetZPlaneCount(uint ZMask)
{
	uint PlaneCount = ZMask;

	if (PlaneCount >= 11u)
	{
		PlaneCount += 2u;
	}
	else if (PlaneCount >= 9u)
	{
		++PlaneCount;
	}

	return PlaneCount;
}

// Note: We don't use MSAA depth, this assumes single sample count
uint PackZLayout(uint ZMask, bool IsTcc)
{
	const uint ZPlaneCount			= GetZPlaneCount(ZMask);
	const uint ZPlaneSize			= ZPlaneCount * 12u;
	const uint ZPlanePaddedSize		= Padding(ZPlaneSize, 32u);
	const uint PMaskBitCount		= CeilLog2(ZPlaneCount);
	const uint PMaskSize			= 64u * PMaskBitCount / 8u;
	const uint PMaskPaddedSize		= Padding(PMaskSize, 32u);
	const uint DepthPaddedSize		= ZPlanePaddedSize + PMaskPaddedSize;
	const uint DepthPackedSize		= Padding(ZPlaneSize + PMaskSize, 32u);

	//
	// Empirically TCC depth has a different PMaskOffset for ZMask == 3 than non-TCC.
	// This is not confirmed by AMD but does appear to work in our engine.
	//

	const uint PMaskOffset =
		Padding(
			select(DepthPackedSize < DepthPaddedSize, ZPlaneSize, ZPlanePaddedSize),
			select(DepthPackedSize > 256u, 256u, 8u))
		+ select(IsTcc && (ZMask == 3), 8u, 0u);

	const uint ZPlaneWordCount		= (PMaskOffset + PMaskSize) >> 2u;

	uint ZLayout = ZPlaneCount;
	ZLayout |= PMaskBitCount << 5;
	ZLayout |= select(DepthPackedSize <= 32, 1u, 0u) << 16;
	ZLayout |= (PMaskOffset >> 2u) << 17u;
	ZLayout |= ZPlaneWordCount << 24u;
	return ZLayout;
}

struct ZLayout
{
	uint ZPlaneCount;
	uint ZPlaneWordCount;
	uint PMaskOffset;
	uint IndexBitCount;
	uint Is32BytePadded;
};

ZLayout UnpackZLayout(uint ZMask, bool IsTcc)
{
	const uint ZLayoutTable[21u] =
	{
		PackZLayout(0x0, false),
		PackZLayout(0x1, false),
		PackZLayout(0x2, false),
		PackZLayout(0x3, false),
		PackZLayout(0x4, false),
		PackZLayout(0x5, false),
		PackZLayout(0x6, false),
		PackZLayout(0x7, false),
		PackZLayout(0x8, false),
		PackZLayout(0x9, false),
		PackZLayout(0xA, false),
		PackZLayout(0xB, false),
		PackZLayout(0xC, false),
		PackZLayout(0xD, false),
		PackZLayout(0xE, false),
		PackZLayout(0xF, false),

		// TCC-depth is limited to 4 planes
		PackZLayout(0x0, true),
		PackZLayout(0x1, true),
		PackZLayout(0x2, true),
		PackZLayout(0x3, true),
		PackZLayout(0x4, true),
	};

	const uint LayoutOffset = IsTcc ? 16u : 0u;
	const uint PackedZLayout = ZLayoutTable[LayoutOffset + ZMask];

	ZLayout Layout;
	Layout.ZPlaneCount		= BitFieldExtractU32(PackedZLayout, 5u, 0u);
	Layout.IndexBitCount	= BitFieldExtractU32(PackedZLayout, 3u, 5u);
	Layout.Is32BytePadded	= BitFieldExtractU32(PackedZLayout, 4u, 13u);
	Layout.PMaskOffset		= BitFieldExtractU32(PackedZLayout, 7u, 17u);
	Layout.ZPlaneWordCount	= BitFieldExtractU32(PackedZLayout, 8u, 24u);
	return Layout;
}

// 112 dword max ever read, but can be offset by 8 words
groupshared uint CompressedDepthData[128];

float DecompressDepthValue_Internal(ZLayout Layout, uint ZOffset, uint ZMask, uint2 PixelPos, uint ThreadIndex, float ClearValue, uint PlatformConfig)
{
	GroupMemoryBarrierWithGroupSync();

	uint ZPlaneIndex = 0;

	BRANCH
	if (Layout.ZPlaneCount > 1) // scalarized
	{
		const uint BitOffset = MulU24(ThreadIndex, Layout.IndexBitCount);
		const uint IndexOffset = ZOffset + Layout.PMaskOffset + (BitOffset >> 5);
		ZPlaneIndex = BitAlignU32(CompressedDepthData[IndexOffset + 1], CompressedDepthData[IndexOffset], BitOffset);
		ZPlaneIndex = BitFieldExtractU32(ZPlaneIndex, Layout.IndexBitCount, 0);
	}

	uint  ZPlaneOffset = (ZPlaneIndex * 3) + ZOffset;
	uint3 ZPlane = uint3(
		CompressedDepthData[ZPlaneOffset + 0u],
		CompressedDepthData[ZPlaneOffset + 1u],
		CompressedDepthData[ZPlaneOffset + 2u]
	);

	int Exponent			= BitFieldExtractU32(ZPlane.y, 8u, 24u);
	int ZCenter				= BitFieldExtractI32(ZPlane.z, 31u, 0u) << 1;

	int2 AdjustedPixelPos	= PixelPos & 7;
	AdjustedPixelPos		= AdjustedPixelPos * 2 - 7;

	int DepthValue = 0;
	DepthValue = MadI24(ZPlane.x >> 4,  AdjustedPixelPos.x, DepthValue     );
	DepthValue = MadI24(ZPlane.y,       AdjustedPixelPos.y, DepthValue     );
	DepthValue = MadI24(ZPlane.x & 0xF, AdjustedPixelPos.x, DepthValue << 4);
	DepthValue = MadI24(ZPlane.x >> 28, AdjustedPixelPos.y, DepthValue     );

	if (DepthValue <= -ZCenter)
	{
		return 0.0f;
	}

	DepthValue += ZCenter;

	uint DepthValueUnsigned = DepthValue;
	int Shift = firstbithigh(DepthValueUnsigned) - 23;
	Exponent += Shift;

	const bool bHandleDenorms = true;
	const bool bFlushDenorms = true;

	if (bHandleDenorms && Exponent < 1)
	{
		return select(bFlushDenorms, 0.0f, asfloat((DepthValueUnsigned << (8 - Shift)) >> (9 - Exponent)));
	}
	else
	{
		return min(1.0f, asfloat(Exponent << 23 | (DepthValueUnsigned << (9 - Shift)) >> 9));
	}
}

float DecompressDepthValue(Texture2D<float> DepthBuffer, uint ZMask, uint2 PixelPos, uint ThreadIndex, float ClearValue, uint PlatformConfig)
{
	if (IsZTileClear(ZMask))
	{
		return ClearValue;
	}

	if (ZMask == HTILE_ZMASK_EXPANDED)
	{
		// Uncompressed, fetch raw depth
		return DepthBuffer[PixelPos];
	}

	const ZLayout Layout = UnpackZLayout(ZMask, IsTccTileConfig(PlatformConfig));
	const uint ZOffset = select(Is32BytePaddedTileConfig(PlatformConfig), Layout.Is32BytePadded & PixelPos.x, 0u);
	const uint MaxThreadIndex = ZOffset + Layout.ZPlaneWordCount;

	if (ThreadIndex < MaxThreadIndex)
	{
		CompressedDepthData[ThreadIndex] = asuint(DepthBuffer[PixelPos]);
	}

	return DecompressDepthValue_Internal(Layout, ZOffset, ZMask, PixelPos, ThreadIndex, ClearValue, PlatformConfig);
}

float DecompressDepthValue(RWTexture2D<float> DepthBuffer, uint ZMask, uint2 PixelPos, uint ThreadIndex, float ClearValue, uint PlatformConfig)
{
	if (IsZTileClear(ZMask))
	{
		return ClearValue;
	}

	if (ZMask == HTILE_ZMASK_EXPANDED)
	{
		// Uncompressed, fetch raw depth
		return DepthBuffer[PixelPos];
	}

	const ZLayout Layout = UnpackZLayout(ZMask, IsTccTileConfig(PlatformConfig));
	const uint ZOffset = select(Is32BytePaddedTileConfig(PlatformConfig), Layout.Is32BytePadded & PixelPos.x, 0u);
	const uint MaxThreadIndex = ZOffset + Layout.ZPlaneWordCount;

	if (ThreadIndex < MaxThreadIndex)
	{
		CompressedDepthData[ThreadIndex] = asuint(DepthBuffer[PixelPos]);
	}

	return DecompressDepthValue_Internal(Layout, ZOffset, ZMask, PixelPos, ThreadIndex, ClearValue, PlatformConfig);
}

uint DecompressStencilValue(Texture2D<uint> StencilBuffer, uint SMem, uint2 PixelPos, uint ClearValue)
{
	// SMem
	// 0 = Clear
	// 1 = Single
	// 2 = Clear and Expanded
	// 3 = Expanded
	if ((SMem & 0x1) == 0)
	{
		return ClearValue;
	}

	if (SMem == 1)
	{
		return StencilBuffer[PixelPos & (uint2)~7u];
	}

	// Expanded
	return StencilBuffer[PixelPos];
}

uint DecompressStencilValue(RWTexture2D<uint> StencilBuffer, uint SMem, uint2 PixelPos, uint ClearValue)
{
	// SMem
	// 0 = Clear
	// 1 = Single
	// 2 = Clear and Expanded
	// 3 = Expanded
	if ((SMem & 0x1) == 0)
	{
		return ClearValue;
	}

	if (SMem == 1)
	{
		return StencilBuffer[PixelPos & (uint2)~7u];
	}

	// Expanded
	return StencilBuffer[PixelPos];
}

uint2 PixelFromThreadIndex(uint Element)
{
	uint2 PixelPos;
	PixelPos.x = ((Element >> 0u) & 1u) | ((Element >> 1u) & 2u) | ((Element >> 2u) & 4u);
	PixelPos.y = ((Element >> 1u) & 1u) | ((Element >> 2u) & 2u) | ((Element >> 3u) & 4u);
	return PixelPos;
}

uint2 SwizzleThreadIndex(uint ThreadIndex)
{
#if 1
	// At least currently, it seems to be faster to just compute the swizzling directly instead of using the LUT.
	return PixelFromThreadIndex(ThreadIndex);
#else
	const uint2 LookupTable[64] =
	{
		PixelFromThreadIndex(8 * 0 + 0),
		PixelFromThreadIndex(8 * 0 + 1),
		PixelFromThreadIndex(8 * 0 + 2),
		PixelFromThreadIndex(8 * 0 + 3),
		PixelFromThreadIndex(8 * 0 + 4),
		PixelFromThreadIndex(8 * 0 + 5),
		PixelFromThreadIndex(8 * 0 + 6),
		PixelFromThreadIndex(8 * 0 + 7),
		PixelFromThreadIndex(8 * 1 + 0),
		PixelFromThreadIndex(8 * 1 + 1),
		PixelFromThreadIndex(8 * 1 + 2),
		PixelFromThreadIndex(8 * 1 + 3),
		PixelFromThreadIndex(8 * 1 + 4),
		PixelFromThreadIndex(8 * 1 + 5),
		PixelFromThreadIndex(8 * 1 + 6),
		PixelFromThreadIndex(8 * 1 + 7),
		PixelFromThreadIndex(8 * 2 + 0),
		PixelFromThreadIndex(8 * 2 + 1),
		PixelFromThreadIndex(8 * 2 + 2),
		PixelFromThreadIndex(8 * 2 + 3),
		PixelFromThreadIndex(8 * 2 + 4),
		PixelFromThreadIndex(8 * 2 + 5),
		PixelFromThreadIndex(8 * 2 + 6),
		PixelFromThreadIndex(8 * 2 + 7),
		PixelFromThreadIndex(8 * 3 + 0),
		PixelFromThreadIndex(8 * 3 + 1),
		PixelFromThreadIndex(8 * 3 + 2),
		PixelFromThreadIndex(8 * 3 + 3),
		PixelFromThreadIndex(8 * 3 + 4),
		PixelFromThreadIndex(8 * 3 + 5),
		PixelFromThreadIndex(8 * 3 + 6),
		PixelFromThreadIndex(8 * 3 + 7),
		PixelFromThreadIndex(8 * 4 + 0),
		PixelFromThreadIndex(8 * 4 + 1),
		PixelFromThreadIndex(8 * 4 + 2),
		PixelFromThreadIndex(8 * 4 + 3),
		PixelFromThreadIndex(8 * 4 + 4),
		PixelFromThreadIndex(8 * 4 + 5),
		PixelFromThreadIndex(8 * 4 + 6),
		PixelFromThreadIndex(8 * 4 + 7),
		PixelFromThreadIndex(8 * 5 + 0),
		PixelFromThreadIndex(8 * 5 + 1),
		PixelFromThreadIndex(8 * 5 + 2),
		PixelFromThreadIndex(8 * 5 + 3),
		PixelFromThreadIndex(8 * 5 + 4),
		PixelFromThreadIndex(8 * 5 + 5),
		PixelFromThreadIndex(8 * 5 + 6),
		PixelFromThreadIndex(8 * 5 + 7),
		PixelFromThreadIndex(8 * 6 + 0),
		PixelFromThreadIndex(8 * 6 + 1),
		PixelFromThreadIndex(8 * 6 + 2),
		PixelFromThreadIndex(8 * 6 + 3),
		PixelFromThreadIndex(8 * 6 + 4),
		PixelFromThreadIndex(8 * 6 + 5),
		PixelFromThreadIndex(8 * 6 + 6),
		PixelFromThreadIndex(8 * 6 + 7),
		PixelFromThreadIndex(8 * 7 + 0),
		PixelFromThreadIndex(8 * 7 + 1),
		PixelFromThreadIndex(8 * 7 + 2),
		PixelFromThreadIndex(8 * 7 + 3),
		PixelFromThreadIndex(8 * 7 + 4),
		PixelFromThreadIndex(8 * 7 + 5),
		PixelFromThreadIndex(8 * 7 + 6),
		PixelFromThreadIndex(8 * 7 + 7),
	};

	return LookupTable[ThreadIndex];
#endif
}

#endif // PLATFORM_SUPPORTS_HTILE_LOOKUP