// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SceneTextureParameters.ush"
#include "ReductionCommon.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"


#define MAX_MIP_BATCH_SIZE 4
#define GROUP_TILE_SIZE 8


float4 DispatchThreadIdToBufferUV;
float2 InvSize;
float2 InputViewportMaxBound;
uint4 PixelViewPortMinMax;	// Max is inclusive
float SceneDepthBias;
float4 InvDeviceZToWorldZTransform;

Texture2D		ParentTextureMip;
SamplerState	ParentTextureMipSampler;

#if PIXELSHADER
uint SourceMipIndex;
#endif

float4 Gather4(Texture2D Texture, SamplerState TextureSampler, float2 BufferUV)
{
	// min(..., InputViewportMaxBound) because we don't want to sample outside of the viewport
	// when the view size has odd dimensions on X/Y axis.
	float2 UV = min(BufferUV + float2(-0.25f, -0.25f) * InvSize, InputViewportMaxBound - InvSize);

#if (PIXELSHADER && DIM_USE_MIPINDEX)
	float4 UVRect = float4(UV, UV + InvSize);
	return float4(
		Texture.SampleLevel(TextureSampler, UVRect.xw, SourceMipIndex).r,
		Texture.SampleLevel(TextureSampler, UVRect.zw, SourceMipIndex).r,
		Texture.SampleLevel(TextureSampler, UVRect.zy, SourceMipIndex).r,
		Texture.SampleLevel(TextureSampler, UVRect.xy, SourceMipIndex).r);
#else
	return Texture.GatherRed(TextureSampler, UV);
#endif
}

#if COMPUTESHADER

RWTexture2D<float> FurthestHZBOutput_0;
RWTexture2D<float> FurthestHZBOutput_1;
RWTexture2D<float> FurthestHZBOutput_2;
RWTexture2D<float> FurthestHZBOutput_3;

RWTexture2D<float> ClosestHZBOutput_0;
RWTexture2D<float> ClosestHZBOutput_1;
RWTexture2D<float> ClosestHZBOutput_2;
RWTexture2D<float> ClosestHZBOutput_3;

groupshared uint SharedMinDeviceZ[GROUP_TILE_SIZE * GROUP_TILE_SIZE];
groupshared float SharedMaxDeviceZ[GROUP_TILE_SIZE * GROUP_TILE_SIZE];

float ConvertFromDeviceZ(float DeviceZ, float4 InvDeviceZToWorldZTransform)
{
	// Supports ortho and perspective, see CreateInvDeviceZToWorldZTransform()
	return DeviceZ * InvDeviceZToWorldZTransform[0] + InvDeviceZToWorldZTransform[1] + 1.0f / (DeviceZ * InvDeviceZToWorldZTransform[2] - InvDeviceZToWorldZTransform[3]);
}

float ConvertToDeviceZ(float SceneDepth, float4 InvDeviceZToWorldZTransform)
{
	// Ortho (no access to View here).
	if (InvDeviceZToWorldZTransform[0] != 0.0f)
	{
		return (SceneDepth - InvDeviceZToWorldZTransform[1] + 1.0f) / InvDeviceZToWorldZTransform[0];
	}
	return 1.0f / ((SceneDepth + InvDeviceZToWorldZTransform[3]) * InvDeviceZToWorldZTransform[2]);
}

float RoundUpF16(float DeviceZ)
{
	// ClosestDeviceZ needs to be rounded up to nearest fp16 to be conservative
	return f16tof32(f32tof16(DeviceZ) + 1);
}

void OutputMipLevel(uint MipLevel, uint2 OutputPixelPos, float FurthestDeviceZ, float ClosestDeviceZ)
{
	#if DIM_MIP_LEVEL_COUNT >= 2
	if (MipLevel == 1)
	{
		#if DIM_FURTHEST
			FurthestHZBOutput_1[OutputPixelPos] = FurthestDeviceZ;
		#endif
		#if DIM_CLOSEST
			ClosestHZBOutput_1[OutputPixelPos] = RoundUpF16(ClosestDeviceZ);
		#endif
	}
	#endif
	#if DIM_MIP_LEVEL_COUNT >= 3
	else if (MipLevel == 2)
	{
		#if DIM_FURTHEST
			FurthestHZBOutput_2[OutputPixelPos] = FurthestDeviceZ;
		#endif
		#if DIM_CLOSEST
			ClosestHZBOutput_2[OutputPixelPos] = RoundUpF16(ClosestDeviceZ);
		#endif
	}
	#endif
	#if DIM_MIP_LEVEL_COUNT >= 4
	else if (MipLevel == 3)
	{
		#if DIM_FURTHEST
			FurthestHZBOutput_3[OutputPixelPos] = FurthestDeviceZ;
		#endif
		#if DIM_CLOSEST
			ClosestHZBOutput_3[OutputPixelPos] = RoundUpF16(ClosestDeviceZ);
		#endif
	}		
	#endif
}

#if VIS_BUFFER_FORMAT == 1

// 64b VisBuffer as uint2
Texture2D<uint2> VisBufferTexture;
float4 Gather4VisZ(float2 BufferUV, uint2 PixelCoord)
{
#if COMPILER_SUPPORTS_GATHER_UINT
	float2 ClampedBufferUV = min( BufferUV + float2(-0.25f, -0.25f) * InvSize, InputViewportMaxBound - InvSize );
	return asfloat(VisBufferTexture.GatherGreen(ParentTextureMipSampler, ClampedBufferUV, 0));
#else
	// Workaround for Gather not being support for R32G32_UINT on D3D11. We can't alias the texture with R3232_FLOAT as that isn't supported in Unreal.
	uint2 ClampedPixelCoord = min(PixelCoord.xy, PixelViewPortMinMax.zw - 1);
	uint4 ClampedPixelCoord4 = uint4(ClampedPixelCoord.xy, ClampedPixelCoord.xy + 1);
	uint4 Out;
	Out.x = VisBufferTexture[ClampedPixelCoord4.xw].g;	// (-, +)
	Out.y = VisBufferTexture[ClampedPixelCoord4.zw].g;	// (+, +)
	Out.z = VisBufferTexture[ClampedPixelCoord4.zy].g;	// (+, -)
	Out.w = VisBufferTexture[ClampedPixelCoord4.xy].g;	// (-, -)
	return asfloat(Out);
#endif
}


#elif VIS_BUFFER_FORMAT == 3

// 64b VisBuffer as UlongType
Texture2D<UlongType> VisBufferTexture;
float4 Gather4VisZ(float2 BufferUV, uint2 PixelCoord)
{
	// Workaround for Gather not being support for R32G32_UINT on D3D11. We can't alias the texture with R3232_FLOAT as that isn't supported in Unreal.
	uint2 ClampedPixelCoord = min(PixelCoord.xy, PixelViewPortMinMax.zw - 1);
	uint4 ClampedPixelCoord4 = uint4(ClampedPixelCoord.xy, ClampedPixelCoord.xy + 1);

	uint4 Out;
	Out.x = UnpackUlongType(VisBufferTexture[ClampedPixelCoord4.xw]).g;	// (-, +)
	Out.y = UnpackUlongType(VisBufferTexture[ClampedPixelCoord4.zw]).g;	// (+, +)
	Out.z = UnpackUlongType(VisBufferTexture[ClampedPixelCoord4.zy]).g;	// (+, -)
	Out.w = UnpackUlongType(VisBufferTexture[ClampedPixelCoord4.xy]).g;	// (-, -)
	return asfloat(Out);
}

#else

// 32b depth only version
Texture2D<uint> VisBufferTexture;


#if VIS_BUFFER_FORMAT == 2

float4 Gather4VisZ(float2 BufferUV, uint2 PixelCoord)
{
#if COMPILER_SUPPORTS_GATHER_UINT
	float2 ClampedBufferUV = min( BufferUV + float2(-0.25f, -0.25f) * InvSize, InputViewportMaxBound - InvSize );
	return asfloat(VisBufferTexture.Gather(ParentTextureMipSampler, ClampedBufferUV, 0));
#else
	uint2 ClampedPixelCoord = min(PixelCoord.xy, PixelViewPortMinMax.zw - 1);
	uint4 ClampedPixelCoord4 = uint4(ClampedPixelCoord.xy, ClampedPixelCoord.xy + 1);
	uint4 Out;
	Out.x = VisBufferTexture[ClampedPixelCoord4.xw].r;	// (-, +)
	Out.y = VisBufferTexture[ClampedPixelCoord4.zw].r;	// (+, +)
	Out.z = VisBufferTexture[ClampedPixelCoord4.zy].r;	// (+, -)
	Out.w = VisBufferTexture[ClampedPixelCoord4.xy].r;	// (-, -)
	return asfloat(Out);
#endif
}

#else

float SampleVisZ(uint2 PixelCoord)
{
	uint2 ClampedPixelCoord = min(PixelCoord.xy, PixelViewPortMinMax.zw - 1);
	uint Out = VisBufferTexture[ClampedPixelCoord.xy].r;	// (-, -)
	return asfloat(Out);
}

#endif

#endif

#if DIM_FROXELS

#define FROXEL_HASH_BUFFER_SIZE (GROUP_TILE_SIZE * GROUP_TILE_SIZE)
#define FROXEL_HASH_THREAD_GROUP_SIZE (GROUP_TILE_SIZE * GROUP_TILE_SIZE)

#define FROXEL_SHARED_HASH_BUFFER_VAR SharedMinDeviceZ

#include "Froxel/FroxelBuild.ush"


#if GROUP_TILE_SIZE != FROXEL_TILE_SIZE
	#error "The froxel construction algorithm assumes the group size matches the tile size, if this is changed this code needs to be updated"
#endif

#endif // DIM_FROXELS


[numthreads(GROUP_TILE_SIZE, GROUP_TILE_SIZE, 1)]
void HZBBuildCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	#if DIM_MIP_LEVEL_COUNT == 1
		uint2 GroupThreadId = uint2(GroupThreadIndex % GROUP_TILE_SIZE, GroupThreadIndex / GROUP_TILE_SIZE);
	#else
		uint2 GroupThreadId = InitialTilePixelPositionForReduction2x2(MAX_MIP_BATCH_SIZE - 1, GroupThreadIndex);
	#endif

	uint2 GroupOffset = GROUP_TILE_SIZE * GroupId;
	uint2 DispatchThreadId = GroupOffset + GroupThreadId;

	// whether or not any of the threads in the group is going to look at valid pixels, a lot don't in the HZB build because of POT rounding.
	uint2 MinPixelCoord = (GroupOffset << 1) + PixelViewPortMinMax.xy;
	bool bValidGroup = all(MinPixelCoord < PixelViewPortMinMax.zw);

	float2 BufferUV = (DispatchThreadId + 0.5) * DispatchThreadIdToBufferUV.xy + DispatchThreadIdToBufferUV.zw;

#if VIS_BUFFER_FORMAT == 4

	// In this case, this is the regular depth buffer (non-nanite) and we must sample using the original coordinates.
	// Note that when we sample with a stride (in case the HZB is smaller, which is the usual case), this is not conservative.
	float4 DeviceZx4 = Gather4(ParentTextureMip, ParentTextureMipSampler, BufferUV * 0.5f);

	uint2 SrcPixelCoord = DispatchThreadId;
	float DeviceZ = SampleVisZ(SrcPixelCoord);
	DeviceZ = saturate(ConvertToDeviceZ(ConvertFromDeviceZ(DeviceZ, InvDeviceZToWorldZTransform) + SceneDepthBias, InvDeviceZToWorldZTransform));
	float MinDeviceZ = max(DeviceZ, min(min3(DeviceZx4.x, DeviceZx4.y, DeviceZx4.z), DeviceZx4.w));
	float MaxDeviceZ = DeviceZ;
#else
	float4 DeviceZ = Gather4(ParentTextureMip, ParentTextureMipSampler, BufferUV);

	uint2 SrcPixelCoord = (DispatchThreadId << 1) + PixelViewPortMinMax.xy;
#if (VIS_BUFFER_FORMAT == 1) || (VIS_BUFFER_FORMAT == 3)
	DeviceZ = max(DeviceZ, Gather4VisZ(BufferUV, SrcPixelCoord));
#elif VIS_BUFFER_FORMAT == 2
	DeviceZ = Gather4VisZ(BufferUV, SrcPixelCoord);
#endif

	float MinDeviceZ = min(min3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);
	float MaxDeviceZ = max(max3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);

#if DIM_FROXELS
	// 2x as each thread does 2x2 through Gather
	uint2 LocalTileId = (2 * GroupThreadId) / (FROXEL_TILE_SIZE);
	uint LocalLinearTileId = LocalTileId.y * 2 + LocalTileId.x;
	 // 2x2 tiles per group
	uint2 GroupTileOffset = GroupId * 2;
	uint2 TileId = GroupTileOffset + LocalTileId;

	if (bValidGroup)
	{
		HashBuildFroxelsFromDeviceZ(DeviceZ, GroupThreadIndex, LocalLinearTileId, GroupTileOffset);
	}
#endif
#endif
	
	uint2 OutputPixelPos = DispatchThreadId;
	
	#if DIM_FURTHEST
		FurthestHZBOutput_0[OutputPixelPos] = MinDeviceZ;
	#endif
	
	#if DIM_CLOSEST
		ClosestHZBOutput_0[OutputPixelPos] = RoundUpF16(MaxDeviceZ);
	#endif

	#if DIM_MIP_LEVEL_COUNT == 1
	{
		// NOP
	}
	#else
	{
		SharedMinDeviceZ[GroupThreadIndex] = asuint(MinDeviceZ);
		SharedMaxDeviceZ[GroupThreadIndex] = MaxDeviceZ;
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		const uint LaneCount = WaveGetLaneCount();
#else
		// Actual wave size is unknown, assume the worst
		const uint LaneCount = 0u;
#endif
	
		UNROLL
		for (uint MipLevel = 1; MipLevel < DIM_MIP_LEVEL_COUNT; ++MipLevel)
		{
			const uint TileSize = uint(GROUP_TILE_SIZE) >> MipLevel;
			const uint ReduceBankSize = TileSize * TileSize;
			
			// More waves than one wrote to LDS, need to sync.
			if ((ReduceBankSize << 2u) > LaneCount)
			{
				GroupMemoryBarrierWithGroupSync();
			}

			BRANCH
			if (GroupThreadIndex < ReduceBankSize)
			{
				float4 ParentMinDeviceZ;
				float4 ParentMaxDeviceZ;
				ParentMinDeviceZ[0] = MinDeviceZ;
				ParentMaxDeviceZ[0] = MaxDeviceZ;

				UNROLL
				for (uint i = 1; i < 4; i++)
				{
					uint LDSIndex = GroupThreadIndex + i * ReduceBankSize;
					ParentMinDeviceZ[i] = asfloat(SharedMinDeviceZ[LDSIndex]);
					ParentMaxDeviceZ[i] = SharedMaxDeviceZ[LDSIndex];
				}
				
				MinDeviceZ = min(min3(ParentMinDeviceZ.x, ParentMinDeviceZ.y, ParentMinDeviceZ.z), ParentMinDeviceZ.w);
				MaxDeviceZ = max(max3(ParentMaxDeviceZ.x, ParentMaxDeviceZ.y, ParentMaxDeviceZ.z), ParentMaxDeviceZ.w);
	
				OutputPixelPos = OutputPixelPos >> 1;
				OutputMipLevel(MipLevel, OutputPixelPos, MinDeviceZ, MaxDeviceZ);
				
				SharedMinDeviceZ[GroupThreadIndex] = asuint(MinDeviceZ);
				SharedMaxDeviceZ[GroupThreadIndex] = MaxDeviceZ;
			}
		}
	}
	#endif
} // HZBBuildCS


#elif PIXELSHADER

void HZBBuildPS(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 BufferUV = SvPosition.xy * DispatchThreadIdToBufferUV.xy + DispatchThreadIdToBufferUV.zw;
	float4 DeviceZ = Gather4(ParentTextureMip, ParentTextureMipSampler, BufferUV);

	float FurthestDeviceZ = min(min(DeviceZ.x, DeviceZ.y), min(DeviceZ.z, DeviceZ.w));

	OutColor = FurthestDeviceZ;
}


#else
	#error Unknown shader frequency

#endif
