// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef USE_HZB_SHARED_SAMPLERS
#define USE_HZB_SHARED_SAMPLERS 1
#endif

#ifndef USE_HZB_HALF_TYPE
#define USE_HZB_HALF_TYPE 0
#endif

// HZB textures (HZB == Furthest HZB)
#if USE_HZB_HALF_TYPE
Texture2D<half> HZBTexture;
#else
Texture2D HZBTexture;
#endif
Texture2D ClosestHZBTexture;
Texture2D FurthestHZBTexture;

// Common parameters
float2 SamplePixelToHZBUV;
float2 ViewportUVToHZBBufferUV;
float4 HZBUvFactorAndInvFactor; // HZBUvFactorAndInvFactor.xy == ViewportUVToHZBBufferUV
float4 HZBUVToScreenUVScaleBias;
float2 HZBBaseTexelSize;
float2 HZBSize;
float2 HZBViewSize; // == HZBViewRectSize
int4   HZBViewRect;
float4 ScreenPosToHZBUVScaleBias;

// 1: if HZB texture has valid content, 0: otherwise
uint bIsHZBValid;
uint bIsFurthestHZBValid;
uint bIsClosestHZBValid;

#if SUPPORTS_INDEPENDENT_SAMPLERS && USE_HZB_SHARED_SAMPLERS
#define HZBSampler GlobalPointClampedSampler
#define HZBTextureSampler GlobalPointClampedSampler
#define FurthestHZBTextureSampler GlobalPointClampedSampler
#define ClosestHZBTextureSampler GlobalPointClampedSampler
#else
SamplerState HZBSampler;
SamplerState HZBTextureSampler;
SamplerState FurthestHZBTextureSampler;	
SamplerState ClosestHZBTextureSampler;
#endif