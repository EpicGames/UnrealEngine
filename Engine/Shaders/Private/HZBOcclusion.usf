// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "Nanite/NaniteHZBCull.ush"

Texture2D		BoundsCenterTexture;
SamplerState	BoundsCenterSampler;
Texture2D		BoundsExtentTexture;
SamplerState	BoundsExtentSampler;

float2 InvTestTargetSize;

void HZBTestPS(in float4 SvPosition : SV_Position, out float4 OutColor : SV_Target0)
{
	float2 InUV = SvPosition.xy * InvTestTargetSize;

	float4 BoundsCenter = BoundsCenterTexture.SampleLevel( BoundsCenterSampler, InUV, 0 );
	float4 BoundsExtent = BoundsExtentTexture.SampleLevel( BoundsExtentSampler, InUV, 0 );
	BoundsCenter.xyz += DFHackToFloat(PrimaryView.PreViewTranslation);

#if 1
	BRANCH
	if( BoundsExtent.w == 0 )
	{
		OutColor = float4( 1, 0, 0, 0 );
		return;
	}
#endif

	const bool bIsOrtho = IsOrthoProjection(View.ViewToClip);
	FFrustumCullData Cull = BoxCullFrustum(BoundsCenter.xyz, BoundsExtent.xyz, View.TranslatedWorldToClip, View.ViewToClip, bIsOrtho, true /* near clip */, false /* skip culling */);
	
	BRANCH
	if( Cull.bIsVisible && !Cull.bCrossesNearPlane )
	{
		FScreenRect Rect = GetScreenRect( int4(0, 0, HZBViewSize), Cull, 4 );

		Cull.bIsVisible = IsVisibleHZB( Rect, true );
	}

	OutColor = Cull.bIsVisible ? 1 : 0;
}