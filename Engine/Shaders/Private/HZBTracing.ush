// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

// HZB mip 0 is half res scene depth. To do accurate traces we need to traverse all the way down to full res scene depth, but this adds an incoherent branch on the inner tracing loop.
#ifndef HZB_TRACE_INCLUDE_FULL_RES_DEPTH
#define HZB_TRACE_INCLUDE_FULL_RES_DEPTH 0
#endif

#ifndef HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH
#define HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH 0
#endif

#if HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH && !HZB_TRACE_INCLUDE_FULL_RES_DEPTH
#	error "HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH is only compatible with HZB_TRACE_INCLUDE_FULL_RES_DEPTH"
#endif

#ifndef TERMINATE_ON_LOW_OCCUPANCY
#define TERMINATE_ON_LOW_OCCUPANCY 0
#endif

/**
 * Accurate but slow screen trace by traversing the HZB.
 * RayTranslatedWorldOrigin must be a position on-screen
 * RayWorldDirection must be unit length
 * OutScreenUV gives either the hit screen UV or the screen UV of the furthest unoccluded position along the ray, which can be plugged into ComputeRayHitSqrDistance.
 * OutLastVisibleScreenUV gives the screen UV of the furthest unoccluded position along the ray, which can be plugged into ComputeRayHitSqrDistance.
 * MaxIterations has a large impact on performance
 * RelativeDepthThickness is how thick to treat each pixel as.  Larger values cause artifacts as the color value of the pixel gets stretched.
 */
void TraceHZB(
	Texture2D InSceneDepthTexture,
	Texture2D InClosestHZBTexture,
	float2 InHZBBaseTexelSize,
	float4 InHZBUVToScreenUVScaleBias,
	float3 RayTranslatedWorldOrigin,
	float3 RayWorldDirection,
	float MaxWorldTraceDistance,
	float4 InHZBUvFactorAndInvFactor,
	float MaxIterations,
	float RelativeDepthThickness,
	float NumThicknessStepsToDetermineCertainty,
	uint MinimumTracingThreadOccupancy,
	inout bool bHit,
	inout bool bUncertain,
	inout float3 OutScreenUV,
	inout float3 OutLastVisibleScreenUV,
	inout float OutHitTileZ)
{
	float3 RayStartScreenUV;
	{
		float4 RayStartClip = mul(float4(RayTranslatedWorldOrigin, 1.0f), View.TranslatedWorldToClip);
		float3 RayStartScreenPosition = RayStartClip.xyz / max(RayStartClip.w, 1.0f);
		RayStartScreenUV = float3((RayStartScreenPosition.xy * float2(0.5f, -0.5f) + 0.5f) * InHZBUvFactorAndInvFactor.xy, RayStartScreenPosition.z);
	}

	float3 RayEndScreenUV;
	{
		float3 ViewRayDirection = mul(float4(RayWorldDirection, 0.0), View.TranslatedWorldToView).xyz;
		float SceneDepth = mul(float4(RayTranslatedWorldOrigin, 1.0f), View.TranslatedWorldToView).z;
		// Clamps the ray to end at the Z == 0 plane so the end point will be valid in NDC space for clipping
		float RayEndWorldDistance = ViewRayDirection.z < 0.0 ? min(-0.99f * SceneDepth / ViewRayDirection.z, MaxWorldTraceDistance) : MaxWorldTraceDistance;

		float3 RayWorldEnd = RayTranslatedWorldOrigin + RayWorldDirection * RayEndWorldDistance;
		float4 RayEndClip = mul(float4(RayWorldEnd, 1.0f), View.TranslatedWorldToClip);
		float3 RayEndScreenPosition = RayEndClip.xyz / RayEndClip.w;
		RayEndScreenUV = float3((RayEndScreenPosition.xy * float2(0.5f, -0.5f) + 0.5f) * InHZBUvFactorAndInvFactor.xy, RayEndScreenPosition.z);

		float2 ScreenEdgeIntersections = LineBoxIntersect(RayStartScreenUV, RayEndScreenUV, float3(0, 0, 0), float3(InHZBUvFactorAndInvFactor.xy, 1));

		// Recalculate end point where it leaves the screen
		RayEndScreenUV = RayStartScreenUV + (RayEndScreenUV - RayStartScreenUV) * ScreenEdgeIntersections.y;
	}

	float BaseMipLevel = HZB_TRACE_INCLUDE_FULL_RES_DEPTH ? -1 : 0;
	float MipLevel = BaseMipLevel;

	float3 RayDirectionScreenUV = RayEndScreenUV - RayStartScreenUV;
	float CurrentIntersectionTime = 0;
	float LastAboveSurfaceTime = 0;

	// Offset to pick which XY boundary planes to intersect
	float2 FloorOffset = select(RayDirectionScreenUV.xy < 0, 0.0, 1.0);

	float3 RayScreenUV = RayStartScreenUV;

	// Step out of current tile without hit test to avoid self-intersection
	bool bStepOutOfCurrentTile = true;

	if (bStepOutOfCurrentTile)
	{
		float MipLevelForStepOut = MipLevel;
		float2 CurrentMipTexelSize = exp2(MipLevelForStepOut) * InHZBBaseTexelSize;
		float2 CurrentMipResolution = 1.0f / CurrentMipTexelSize;

		float2 UVOffset = .005f * CurrentMipTexelSize;
		UVOffset = select(RayDirectionScreenUV.xy < 0, -UVOffset, UVOffset);

		float2 XYPlane = floor(RayScreenUV.xy * CurrentMipResolution) + FloorOffset;
		XYPlane = XYPlane * CurrentMipTexelSize + UVOffset;

		float2 PlaneIntersectionTimes = (XYPlane - RayStartScreenUV.xy) / RayDirectionScreenUV.xy;
		float IntersectionTime = min(PlaneIntersectionTimes.x, PlaneIntersectionTimes.y);
		CurrentIntersectionTime = IntersectionTime;
		RayScreenUV = RayStartScreenUV + CurrentIntersectionTime * RayDirectionScreenUV;
	}

	if (CurrentIntersectionTime >= 1.0f)
	{
		// If bStepOutOfCurrentTile caused the ray to terminate rewind to the start of the ray since screen trace will be skipped
		RayScreenUV = RayStartScreenUV;
	}

	float NumIterations = 0;
	bHit = false;
	bUncertain = false;
	OutHitTileZ = 0;

	// Stackless HZB traversal
	while (MipLevel >= BaseMipLevel
		&& NumIterations < MaxIterations
		&& CurrentIntersectionTime < 1.0f
#if TERMINATE_ON_LOW_OCCUPANCY
		&& WaveActiveCountBits(true) > MinimumTracingThreadOccupancy
#endif
		)
	{
		float2 CurrentMipTexelSize = exp2(MipLevel) * InHZBBaseTexelSize;
		float2 CurrentMipResolution = 1.0f / CurrentMipTexelSize;

		float2 UVOffset = .005f * CurrentMipTexelSize;
		UVOffset = select(RayDirectionScreenUV.xy < 0, -UVOffset, UVOffset);

		float2 XYPlane = floor(RayScreenUV.xy * CurrentMipResolution) + FloorOffset;
		XYPlane = XYPlane * CurrentMipTexelSize + UVOffset;

		float TileZ;

#if HZB_TRACE_INCLUDE_FULL_RES_DEPTH
		if (MipLevel < 0)
		{
			TileZ = InSceneDepthTexture.SampleLevel(GlobalPointClampedSampler, RayScreenUV.xy * InHZBUVToScreenUVScaleBias.xy + InHZBUVToScreenUVScaleBias.zw, 0).r;

#	if HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH
			float TileZFiltered = InSceneDepthTexture.SampleLevel(GlobalBilinearClampedSampler, RayScreenUV.xy * InHZBUVToScreenUVScaleBias.xy + InHZBUVToScreenUVScaleBias.zw, 0).r;

			// Using both point and bilinear filtered depth helps avoid incorrect self shadowing by approximating the underlying surface
			TileZ = min(TileZ, TileZFiltered);
#	endif
		}
		else
#endif
		{
			TileZ = InClosestHZBTexture.SampleLevel(GlobalPointClampedSampler, RayScreenUV.xy, MipLevel).x;
#if !HZB_TRACE_INCLUDE_FULL_RES_DEPTH
			TileZ *= lerp(.99f, 1.0f, saturate(CurrentIntersectionTime * 10.0f));
#endif
		}

		float3 BoundaryPlanes = float3(XYPlane, TileZ);

		float3 PlaneIntersectionTimes = (BoundaryPlanes - RayStartScreenUV) / RayDirectionScreenUV;
		PlaneIntersectionTimes.z = RayDirectionScreenUV.z < 0 ? PlaneIntersectionTimes.z : 1.0f;
		float IntersectionTime = min(PlaneIntersectionTimes.x, PlaneIntersectionTimes.y);

		bool bAboveSurface = RayScreenUV.z > TileZ;
		bool bSkippedTile = bAboveSurface;

#if HZB_TRACE_USE_BILINEAR_SAMPLED_DEPTH
		if (MipLevel > BaseMipLevel) // don't intersect with Z-plane when using BaseMipLevel since it causes incorrect self shadowing
#endif
		{
			IntersectionTime = min(IntersectionTime, PlaneIntersectionTimes.z);
			bSkippedTile = bSkippedTile && (IntersectionTime != PlaneIntersectionTimes.z);
		}

		if (bSkippedTile)
		{
			LastAboveSurfaceTime = IntersectionTime;
		}

		CurrentIntersectionTime = bAboveSurface ? IntersectionTime : CurrentIntersectionTime;
		RayScreenUV = RayStartScreenUV + min(CurrentIntersectionTime, 1.0f) * RayDirectionScreenUV;
		MipLevel += bSkippedTile ? 1 : -1;

		NumIterations++;
	}

	if (MipLevel < BaseMipLevel && CurrentIntersectionTime < 1.0f)
	{
		float TileZ;

#if HZB_TRACE_INCLUDE_FULL_RES_DEPTH
		TileZ = InSceneDepthTexture.SampleLevel(GlobalPointClampedSampler, RayScreenUV.xy * InHZBUVToScreenUVScaleBias.xy + InHZBUVToScreenUVScaleBias.zw, 0).r;
#else
		TileZ = InClosestHZBTexture.SampleLevel(GlobalPointClampedSampler, RayScreenUV.xy, 0).x;
#endif

		OutHitTileZ = TileZ;

		float HitSceneDepth = ConvertFromDeviceZ(TileZ);
		float RaySceneDepth = ConvertFromDeviceZ(RayScreenUV.z);

		bHit = (RaySceneDepth - HitSceneDepth) < RelativeDepthThickness * max(HitSceneDepth, .00001f);

		if (!bHit)
		{
			// We went below the surface and couldn't count it as a hit, rewind to the last time we were above
			RayScreenUV = RayStartScreenUV + LastAboveSurfaceTime * RayDirectionScreenUV;
		}
	}

	// Linear steps to determine feature thickness along the ray, to reject hits behind very thin surfaces (grass / hair / foliage)
	if (bHit && !bUncertain && NumThicknessStepsToDetermineCertainty > 0)
	{
		float ThicknessSearchMipLevel = 0.0f;
		float MipNumTexels = exp2(ThicknessSearchMipLevel);
		float2 HZBTileSize = MipNumTexels * InHZBBaseTexelSize;
		float NumSteps = NumThicknessStepsToDetermineCertainty / MipNumTexels;
		float ThicknessSearchEndTime = min(length(RayDirectionScreenUV.xy * HZBTileSize * NumSteps) / length(RayEndScreenUV.xy - RayScreenUV.xy), 1.0f);

		for (float I = 0; I < NumSteps; I++)
		{
			float3 SampleUV = RayScreenUV + (I / NumSteps) * ThicknessSearchEndTime * (RayEndScreenUV - RayScreenUV);

			if (all(and (SampleUV.xy > 0, SampleUV.xy < InHZBUvFactorAndInvFactor.xy)))
			{
				float SampleTileZ = InClosestHZBTexture.SampleLevel(GlobalPointClampedSampler, SampleUV.xy, ThicknessSearchMipLevel).x;

				if (SampleUV.z > SampleTileZ)
				{
					bUncertain = true;
				}
			}
		}
	}

	// Debug - visualize clipped endpoints
	//RayScreenUV = RayEndScreenUV;
	//bHit = bUncertain;

	// Output in GBuffer SceneUV space for xy
	OutScreenUV.xy = RayScreenUV.xy * InHZBUVToScreenUVScaleBias.xy + InHZBUVToScreenUVScaleBias.zw;
	OutScreenUV.z = RayScreenUV.z;

	float3 LastVisibleScreenUV = RayStartScreenUV + LastAboveSurfaceTime * RayDirectionScreenUV;
	OutLastVisibleScreenUV.xy = LastVisibleScreenUV.xy * InHZBUVToScreenUVScaleBias.xy + InHZBUVToScreenUVScaleBias.zw;
	OutLastVisibleScreenUV.z = LastVisibleScreenUV.z;
}