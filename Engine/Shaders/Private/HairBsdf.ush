// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairShadingCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Utility functions

float Hair_g(float B, float Theta, bool bClampBSDFValue)
{
	// Clamp B for the denominator term, as otherwise the Gaussian normalization returns too high value.
	// This clamps allow to prevent large value for low roughness, while keeping the highlight shape/sharpness 
	// similar.
	const float DenominatorB = bClampBSDFValue ? max(B, 0.01f) : B;
	return exp(-0.5 * Pow2(Theta) / (B * B)) / (sqrt(2 * PI) * DenominatorB);
}

float Hair_F(float CosTheta)
{
	const float n = 1.55;
	const float F0 = Pow2((1 - n) / (1 + n));
	return F0 + (1 - F0) * Pow5(1 - CosTheta);
}

float3 KajiyaKayDiffuseAttenuation(FGBufferData GBuffer, float3 L, float3 V, half3 N, float Shadow)
{
	// Use soft Kajiya Kay diffuse attenuation
	float KajiyaDiffuse = 1 - abs(dot(N, L));

	float3 FakeNormal = normalize(V - N * dot(V, N));
	//N = normalize( DiffuseN + FakeNormal * 2 );
	N = FakeNormal;

	// Hack approximation for multiple scattering.
	float MinValue = 0.0001f;
	float Wrap = 1;
	float NoL = saturate((dot(N, L) + Wrap) / Square(1 + Wrap));
	float DiffuseScatter = (1 / PI) * lerp(NoL, KajiyaDiffuse, 0.33) * GBuffer.Metallic;
	float Luma = Luminance(GBuffer.BaseColor);
    float3 BaseOverLuma = abs(GBuffer.BaseColor / max(Luma, MinValue));
	float3 ScatterTint = Shadow < 1 ? pow(BaseOverLuma, 1 - Shadow) : 1;
	return sqrt(abs(GBuffer.BaseColor)) * DiffuseScatter * ScatterTint;
}

float3 EvaluateHairMultipleScattering(
	const FHairTransmittanceData TransmittanceData,
	const float Roughness,
	const float3 Fs)
{
	return TransmittanceData.GlobalScattering * (Fs + TransmittanceData.LocalScattering) * TransmittanceData.OpaqueVisibility;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair BSDF Reference code

#define HAIR_REFERENCE 0
#if HAIR_REFERENCE

struct FHairTemp
{
	float SinThetaL;
	float SinThetaV;
	float CosThetaD;
	float CosThetaT;
	float CosPhi;
	float CosHalfPhi;
	float VoL;
	float n_prime;
};

// Modified Bessel function
float I0(float x)
{
	x = abs(x);
	float a;
	if (x < 3.75)
	{
		float t = x / 3.75;
		float t2 = t * t;
		a = +0.0045813;
		a = a * t2 + 0.0360768;
		a = a * t2 + 0.2659732;
		a = a * t2 + 1.2067492;
		a = a * t2 + 3.0899424;
		a = a * t2 + 3.5156229;
		a = a * t2 + 1.0;
	}
	else
	{
		float t = 3.75 / x;
		a = +0.00392377;
		a = a * t - 0.01647633;
		a = a * t + 0.02635537;
		a = a * t - 0.02057706;
		a = a * t + 0.00916281;
		a = a * t - 0.00157565;
		a = a * t + 0.00225319;
		a = a * t + 0.01328592;
		a = a * t + 0.39894228;
		a *= exp(x) * rsqrt(x);
	}
	return a;
}

float LongitudinalScattering(float B, float SinThetaL, float SinThetaV)
{
	float v = B * B;
	float CosThetaL2 = 1 - SinThetaL * SinThetaL;
	float CosThetaV2 = 1 - SinThetaV * SinThetaV;
	float Mp = 0;
	if (v < 0.1)
	{
		float a = sqrt(CosThetaL2 * CosThetaV2) / v;
		float b = -SinThetaL * SinThetaV / v;
		float logI0a = a > 12 ? a + 0.5 * (-log(2 * PI) + log(1 / a) + 0.125 / a) : log(I0(a));
		Mp = exp(logI0a + b - rcp(v) + 0.6931 + log(0.5 / v));
	}
	else
	{
		Mp = rcp(exp(2 / v) * v - v) * exp((1 - SinThetaL * SinThetaV) / v) * I0(sqrt(CosThetaL2 * CosThetaV2) / v);
	}

	return Mp;
}

float GaussianDetector(float Bp, float Phi)
{
	float Dp = 0;
	for (int k = -4; k <= 4; k++)
	{
		// TODO use symmetry and detect for both Phi and -Phi
		Dp += Hair_g(Bp, Phi - (2 * PI) * k, false);
	}
	return Dp;
}

float3 Attenuation(uint p, float h, float3 Color, FHairTemp HairTemp)
{
	float3 A;
	if (p == 0)
	{
		//A = F( cos( 0.5 * acos( HairTemp.VoL ) ) );
		A = Hair_F(sqrt(0.5 + 0.5 * HairTemp.VoL));
	}
	else
	{
		// ua is absorption
		// ua = pe*Sigma_ae + pp*Sigma_ap
		float3 Sigma_ae = { 0.419, 0.697, 1.37 };
		float3 Sigma_ap = { 0.187, 0.4, 1.05 };
		//float3 ua = 0.25 * Sigma_ae + 0.25 * Sigma_ap;
		float3 ua = -0.25 * log(Color);
		float3 ua_prime = ua / HairTemp.CosThetaT;
		//float3 ua_prime = ua / sqrt( 1 - Pow2( HairTemp.CosThetaD ) / 2.4 );

		float yi = asin(h);
		float yt = asin(h / HairTemp.n_prime);

		float f = Hair_F(HairTemp.CosThetaD * sqrt(1 - h * h));		// (14)
		//float3 T = exp( -2 * ua_prime * ( 1 + cos(2*yt) ) );
		float3 T = exp(-2 * ua_prime * cos(yt));
		if (p == 1)
			A = Pow2(1 - f) * T;		// (13)
		else
			A = Pow2(1 - f) * f * T * T;	// (13)
	}
	return A;
}

float Omega(uint p, float h, FHairTemp HairTemp)
{
	float yi = asin(h);
	float yt = asin(h / HairTemp.n_prime);
	return 2 * p * yt - 2 * yi + p * PI;
}

float3 AzimuthalScattering(uint p, float Bp, float3 Color, FHairTemp HairTemp, uint2 Random)
{
	float Phi = acos(HairTemp.CosPhi);

	// Np = 0.5 * Integral_-1^1( A(p,h) * Dp( Phi - Omega(p,h) ) * dh )

	float Offset = float(Random.x & 0xffff) / (1 << 16);

	uint Num = 16;
	float3 Np = 0;
	for (uint i = 0; i < Num; i++)
	{
		float h = ((float)(i + Offset) / Num) * 2 - 1;
		Np += Attenuation(p, h, Color, HairTemp) * GaussianDetector(Bp, Phi - Omega(p, h, HairTemp));
	}
	Np *= 2.0 / Num;

	return 0.5 * Np;
}

// [d'Eon et al. 2011, "An Energy-Conserving Hair Reflectance Model"]
// [d'Eon et al. 2014, "A Fiber Scattering Model with Non-Separable Lobes"]
float3 HairShadingRef(FGBufferData GBuffer, float3 L, float3 V, half3 N, uint2 Random, uint HairComponents)
{
	// to prevent NaN with decals
	// OR-18489 HERO: IGGY: RMB on E ability causes blinding hair effect
	// OR-17578 HERO: HAMMER: E causes blinding light on heroes with hair
	float ClampedRoughness = clamp(GBuffer.Roughness, 1 / 255.0f, 1.0f);

	float n = 1.55;

	FHairTemp HairTemp;

	// N is the vector parallel to hair pointing toward root
	HairTemp.VoL = dot(V, L);
	HairTemp.SinThetaL = dot(N, L);
	HairTemp.SinThetaV = dot(N, V);
	// SinThetaT = 1/n * SinThetaL
	HairTemp.CosThetaT = sqrt(1 - Pow2((1 / n) * HairTemp.SinThetaL));
	HairTemp.CosThetaD = cos(0.5 * abs(asin(HairTemp.SinThetaV) - asin(HairTemp.SinThetaL)));

	float3 Lp = L - HairTemp.SinThetaL * N;
	float3 Vp = V - HairTemp.SinThetaV * N;
	HairTemp.CosPhi = dot(Lp, Vp) * rsqrt(dot(Lp, Lp) * dot(Vp, Vp));
	HairTemp.CosHalfPhi = sqrt(0.5 + 0.5 * HairTemp.CosPhi);

	HairTemp.n_prime = sqrt(n * n - 1 + Pow2(HairTemp.CosThetaD)) / HairTemp.CosThetaD;

	float Shift = 0.035;
	float Alpha[] =
	{
		-Shift * 2,
		Shift,
		Shift * 4,
	};
	float B[] =
	{
		Pow2(ClampedRoughness),
		Pow2(ClampedRoughness) / 2,
		Pow2(ClampedRoughness) * 2,
	};

	float3 S = 0;
	UNROLL for (uint p = 0; p < 3; p++)
	{
		if (p == 0 && (HairComponents & HAIR_COMPONENT_R) == 0) continue;
		if (p == 1 && (HairComponents & HAIR_COMPONENT_TT) == 0) continue;
		if (p == 2 && (HairComponents & HAIR_COMPONENT_TRT) == 0) continue;

		float SinThetaV = HairTemp.SinThetaV;
		float Bp = B[p];
		if (p == 0)
		{
			Bp *= sqrt(2.0) * HairTemp.CosHalfPhi;
			float sa, ca;
			sincos(Alpha[p], sa, ca);
			SinThetaV -= 2 * sa * (HairTemp.CosHalfPhi * ca * sqrt(1 - SinThetaV * SinThetaV) + sa * SinThetaV);
		}
		else
		{
			SinThetaV = sin(asin(SinThetaV) - Alpha[p]);
		}
		float Mp = LongitudinalScattering(Bp, HairTemp.SinThetaL, SinThetaV);
		float3 Np = AzimuthalScattering(p, B[p], GBuffer.BaseColor, HairTemp, Random);

		float3 Sp = Mp * Np;
		S += Sp;
	}
	return S;
}
#endif


///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair BSDF
// Approximation to HairShadingRef using concepts from the following papers:
// [Marschner et al. 2003, "Light Scattering from Human Hair Fibers"]
// [Pekelis et al. 2015, "A Data-Driven Light Scattering Model for Hair"]

float3 HairShading( FGBufferData GBuffer, float3 L, float3 V, half3 N, float Shadow, FHairTransmittanceData HairTransmittance, float InBacklit, float Area, uint2 Random )
{
	// to prevent NaN with decals
	// OR-18489 HERO: IGGY: RMB on E ability causes blinding hair effect
	// OR-17578 HERO: HAMMER: E causes blinding light on heroes with hair
	float ClampedRoughness = clamp(GBuffer.Roughness, 1/255.0f, 1.0f);

	//const float3 DiffuseN	= OctahedronToUnitVector( GBuffer.CustomData.xy * 2 - 1 );
	const float Backlit	= min(InBacklit, HairTransmittance.bUseBacklit ? GBuffer.CustomData.z : 1);

#if HAIR_REFERENCE
	// todo: ClampedRoughness is missing for this code path
	float3 S = HairShadingRef( GBuffer, L, V, N, Random );
	//float3 S = HairShadingMarschner( GBuffer, L, V, N );
#else
	// N is the vector parallel to hair pointing toward root

	const float VoL       = dot(V,L);                                                      
	const float SinThetaL = clamp(dot(N,L), -1.f, 1.f);
	const float SinThetaV = clamp(dot(N,V), -1.f, 1.f);
	float CosThetaD = cos( 0.5 * abs( asinFast( SinThetaV ) - asinFast( SinThetaL ) ) );

	//CosThetaD = abs( CosThetaD ) < 0.01 ? 0.01 : CosThetaD;

	const float3 Lp = L - SinThetaL * N;
	const float3 Vp = V - SinThetaV * N;
	const float CosPhi = dot(Lp,Vp) * rsqrt( dot(Lp,Lp) * dot(Vp,Vp) + 1e-4 );
	const float CosHalfPhi = sqrt( saturate( 0.5 + 0.5 * CosPhi ) );
	//const float Phi = acosFast( CosPhi );

	float n = 1.55;
	//float n_prime = sqrt( n*n - 1 + Pow2( CosThetaD ) ) / CosThetaD;
	float n_prime = 1.19 / CosThetaD + 0.36 * CosThetaD;

	float Shift = 0.035;
	float Alpha[] =
	{
		-Shift * 2,
		Shift,
		Shift * 4,
	};	
	float B[] =
	{
		Area + Pow2(ClampedRoughness),
		Area + Pow2(ClampedRoughness) / 2,
		Area + Pow2(ClampedRoughness) * 2,
	};

	float3 S = 0;
	if (HairTransmittance.ScatteringComponent & HAIR_COMPONENT_R)
	{
		const float sa = sin(Alpha[0]);
		const float ca = cos(Alpha[0]);
		float ShiftR = 2 * sa * (ca * CosHalfPhi * sqrt(1 - SinThetaV * SinThetaV) + sa * SinThetaV);
		float BScale = HairTransmittance.bUseSeparableR ? sqrt(2.0) * CosHalfPhi : 1;
		float Mp = Hair_g(B[0] * BScale, SinThetaL + SinThetaV - ShiftR, HairTransmittance.bClampBSDFValue);
		float Np = 0.25 * CosHalfPhi;
		float Fp = Hair_F(sqrt(saturate(0.5 + 0.5 * VoL)));
		S += Mp * Np * Fp * (GBuffer.Specular * 2) * lerp(1, Backlit, saturate(-VoL));
	}

	// TT
	if (HairTransmittance.ScatteringComponent & HAIR_COMPONENT_TT)
	{
		float Mp = Hair_g( B[1], SinThetaL + SinThetaV - Alpha[1], HairTransmittance.bClampBSDFValue);

		float a = 1 / n_prime;
		//float h = CosHalfPhi * rsqrt( 1 + a*a - 2*a * sqrt( 0.5 - 0.5 * CosPhi ) );
		//float h = CosHalfPhi * ( ( 1 - Pow2( CosHalfPhi ) ) * a + 1 );
		float h = CosHalfPhi * ( 1 + a * ( 0.6 - 0.8 * CosPhi ) );
		//float h = 0.4;
		//float yi = asinFast(h);
		//float yt = asinFast(h / n_prime);
		
		float f = Hair_F( CosThetaD * sqrt( saturate( 1 - h*h ) ) );
		float Fp = Pow2(1 - f);
		//float3 Tp = pow( GBuffer.BaseColor, 0.5 * ( 1 + cos(2*yt) ) / CosThetaD );
		//float3 Tp = pow( GBuffer.BaseColor, 0.5 * cos(yt) / CosThetaD );
		float3 Tp = 0;
		if (HairTransmittance.bUseLegacyAbsorption)
		{
			Tp = pow(abs(GBuffer.BaseColor), 0.5 * sqrt(1 - Pow2(h * a)) / CosThetaD);
		}
		else
		{
			// Compute absorption color which would match user intent after multiple scattering
			const float3 AbsorptionColor = HairColorToAbsorption(GBuffer.BaseColor);
			Tp = exp(-AbsorptionColor * 2 * abs(1 - Pow2(h * a) / CosThetaD));
		}

		//float t = asin( 1 / n_prime );
		//float d = ( sqrt(2) - t ) / ( 1 - t );
		//float s = -0.5 * PI * (1 - 1 / n_prime) * log( 2*d - 1 - 2 * sqrt( d * (d - 1) ) );
		//float s = 0.35;
		//float Np = exp( (Phi - PI) / s ) / ( s * Pow2( 1 + exp( (Phi - PI) / s ) ) );
		//float Np = 0.71 * exp( -1.65 * Pow2(Phi - PI) );
		float Np = exp( -3.65 * CosPhi - 3.98 );

		S += Mp * Np * Fp * Tp * Backlit;
	}

	// TRT
	if (HairTransmittance.ScatteringComponent & HAIR_COMPONENT_TRT)
	{
		float Mp = Hair_g( B[2], SinThetaL + SinThetaV - Alpha[2], HairTransmittance.bClampBSDFValue);
		
		//float h = 0.75;
		float f = Hair_F( CosThetaD * 0.5 );
		float Fp = Pow2(1 - f) * f;
		//float3 Tp = pow( GBuffer.BaseColor, 1.6 / CosThetaD );
		float3 Tp = pow(abs(GBuffer.BaseColor), 0.8 / CosThetaD );

		//float s = 0.15;
		//float Np = 0.75 * exp( Phi / s ) / ( s * Pow2( 1 + exp( Phi / s ) ) );
		float Np = exp( 17 * CosPhi - 16.78 );

		S += Mp * Np * Fp * Tp;
	}
#endif

	if (HairTransmittance.ScatteringComponent & HAIR_COMPONENT_MULTISCATTER)
	{
		S  = EvaluateHairMultipleScattering(HairTransmittance, ClampedRoughness, S);
		S += KajiyaKayDiffuseAttenuation(GBuffer, L, V, N, Shadow);
	}

	S = -min(-S, 0.0);
	return S;
}

