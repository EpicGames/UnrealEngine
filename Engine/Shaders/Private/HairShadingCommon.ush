// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair components

// Hair reflectance component (R, TT, TRT, Local Scattering, Global Scattering, Multi Scattering,...)
#define HAIR_COMPONENT_R			0x1u
#define HAIR_COMPONENT_TT			0x2u
#define HAIR_COMPONENT_TRT			0x4u
#define HAIR_COMPONENT_LS			0x8u 
#define HAIR_COMPONENT_GS			0x10u
#define HAIR_COMPONENT_MULTISCATTER	0x20u
#define HAIR_COMPONENT_TT_MODEL  	0x40u

///////////////////////////////////////////////////////////////////////////////////////////////////
// Transmittance functions

struct FHairTransmittanceData
{
	bool bUseLegacyAbsorption;
	bool bUseSeparableR;
	bool bUseBacklit;
	bool bClampBSDFValue;

	float  OpaqueVisibility;
	float3 LocalScattering;
	float3 GlobalScattering;

	uint ScatteringComponent;
};

FHairTransmittanceData InitHairTransmittanceData(bool bMultipleScatterEnable = true)
{
	FHairTransmittanceData o;
	o.bUseLegacyAbsorption = true;
	o.bUseSeparableR = true;
	o.bUseBacklit = false;
	o.bClampBSDFValue = true;

	o.OpaqueVisibility = 1;
	o.LocalScattering = 0;
	o.GlobalScattering = 1;
	o.ScatteringComponent = HAIR_COMPONENT_R | HAIR_COMPONENT_TT | HAIR_COMPONENT_TRT | (bMultipleScatterEnable ? HAIR_COMPONENT_MULTISCATTER : 0);

	return o;
}

FHairTransmittanceData InitHairStrandsTransmittanceData(bool bMultipleScatterEnable = false)
{
	FHairTransmittanceData o = InitHairTransmittanceData(bMultipleScatterEnable);
	o.bUseLegacyAbsorption = false;
	o.bUseBacklit = true;
	return o;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair Absorption
 
// Reference: A Practical and Controllable Hair and Fur Model for Production Path Tracing.
float3 HairAbsorptionToColor(float3 A, float B=0.3f)
{
	const float b2 = B * B;
	const float b3 = B * b2;
	const float b4 = b2 * b2;
	const float b5 = B * b4;
	const float D = (5.969f - 0.215f * B + 2.532f * b2 - 10.73f * b3 + 5.574f * b4 + 0.245f * b5);
	return exp(-sqrt(A) * D);
}

// Reference: A Practical and Controllable Hair and Fur Model for Production Path Tracing.
float3 HairColorToAbsorption(float3 C, float B = 0.3f)
{
	const float b2 = B * B;
	const float b3 = B * b2;
	const float b4 = b2 * b2;
	const float b5 = B * b4;
	const float D = (5.969f - 0.215f * B + 2.532f * b2 - 10.73f * b3 + 5.574f * b4 + 0.245f * b5);
	return Pow2(log(C) / D);
}

// Reference: An Energy-Conserving Hair Reflectance Model
// Adapated for [0..1] range
float3 GetHairColorFromMelanin(float InMelanin, float InRedness, float3 InDyeColor)
{
	InMelanin = saturate(InMelanin);
	InRedness = saturate(InRedness);
	const float Melanin		= -log(max(1 - InMelanin, 0.0001f));
	const float Eumelanin 	= Melanin * (1 - InRedness);
	const float Pheomelanin = Melanin * InRedness;

	const float3 DyeAbsorption = HairColorToAbsorption(saturate(InDyeColor));
	const float3 Absorption = Eumelanin * float3(0.506f, 0.841f, 1.653f) + Pheomelanin * float3(0.343f, 0.733f, 1.924f);

	return HairAbsorptionToColor(Absorption + DyeAbsorption);
}