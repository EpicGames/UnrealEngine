// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define HAIR_CUSTOM_BSDF 1

#include "../Common.ush"
#include "../ShadingCommon.ush"
#include "../ShadingModels.ush"
#include "../HairBsdf.ush"


float2 SampleMp(float2 u, float Roughness, float cosThetaO, float sinThetaO)
{
	const float v = Pow2(Roughness); // Roughness to variance

	u.x = max(u.x, float(1e-5));

	float cosTheta = 1 + v * log(u.x + (1 - u.x) * exp(-2 / v));
	float sinTheta = sqrt(max(0.f, 1 - Pow2(cosTheta)));
	float cosPhi = cos(2 * PI * u.y);
	float sinThetaI = -cosTheta * sinThetaO + sinTheta * cosPhi * cosThetaO;
	float cosThetaI = sqrt(max(0.f, 1 - Pow2(sinThetaI)));

	return float2(cosThetaI, sinThetaI);
}

float Phi(int p, float gammaO, float gammaT)
{
	return 2 * p * gammaT - 2 * gammaO + p * PI;
}

float3 SampleNp(float h, float cosThetaO, float sinThetaO)
{
	const float eta = 1.55f;
	const float s = 0.11715981f; // For radial roughness of 0.3
	const float gammaO = asin(h);

	// Compute $\gammat$ for refracted ray
	float etap = sqrt(eta * eta - Pow2(sinThetaO)) / cosThetaO;
	float sinGammaT = h / etap;
	float gammaT = asin(sinGammaT);
	return float3(
		Phi(0, gammaO, gammaT),
		Phi(1, gammaO, gammaT),
		Phi(2, gammaO, gammaT));
	// +SampleTrimmedLogistic(u, s, -PI, PI);
}

float3 asinFast3(float3 a)
{
	return float3(
		asinFast(a.x),
		asinFast(a.y),
		asinFast(a.z));
}

void SampleHair(
	float Roughness,
	float3 BaseColor,
	float Specular,
	float Backlit,
	float3 V,
	float3 T,
	float2 u0,
	float2 u1,
	uint HairComponents,
	inout float3 OutDirection,
	inout float3 OutWeight)
{
	//float h = (floor(hh*4)+0.5f)/4;
	const float h = clamp(u1.x * 2 - 1, -1, 1);

	const float Area = 0;
	const float ClampedRoughness = clamp(Roughness, 1 / 255.0f, 1.0f);

	const float SinThetaV = dot(T, V);
	const float CosThetaV = sqrt(1 - SinThetaV * SinThetaV);
	const float phiO = atan2(V.z, V.y);

	const float3 dPhiL = SampleNp(h, CosThetaV, SinThetaV);
	const float3 CosPhi = cos(dPhiL);
	const float3 CosHalfPhi = cos(0.5f * dPhiL);

	float Shift = 0.035;

	const float sa = sin(-Shift * 2);
	const float ca = cos(-Shift * 2);
	const float ShiftR = 2 * sa* (ca * CosHalfPhi[0] * sqrt(1 - SinThetaV * SinThetaV) + sa * SinThetaV);

	float3 Alpha = float3
	(
		ShiftR,
		Shift,
		Shift * 4
	);
	float B[] =
	{
		Area + Pow2(ClampedRoughness),// * sqrt(CosHalfPhi[0] * sqrt(2)),
		Area + Pow2(ClampedRoughness) / 2,
		Area + Pow2(ClampedRoughness) * 2,
	};

	// 1. Take mirror direction + offset
	const float3 SinThetaL = sin(asin(SinThetaV) + Alpha);
	const float3 CosThetaD = cos(0.5 * abs(asinFast(SinThetaV).xxx - asinFast3(SinThetaL)));


	uint p = 0;
	float3 Ap[3];
	Ap[0] = 0.0;
	Ap[1] = 0.0;
	Ap[2] = 0.0;

	// 2. Lobe selection
	// R Energy
	if (HairComponents & HAIR_COMPONENT_R)
	{
		const float VoL = cos(abs(asinFast(SinThetaV) - asinFast(SinThetaL[0])));

		float Np = 0.25 * CosHalfPhi[0];
		float Fp = Hair_F(sqrt(saturate(0.5 + 0.5 * VoL)));

		Ap[0] = Np * Fp * (Specular * 2);
	}
	// TT Energy
	if (HairComponents & HAIR_COMPONENT_TT)
	{
		const float n = 1.55;
		const float n_prime = 1.19 / CosThetaD[1] + 0.36 * CosThetaD[1];
		float a = 1 / n_prime;
		float h = CosHalfPhi[1] * (1 + a * (0.6 - 0.8 * CosPhi[1]));
		float f = Hair_F(CosThetaD[1] * sqrt(saturate(1 - h * h)));
		float Fp = Pow2(1 - f);
		float3 Tp = pow(BaseColor, 0.5 * sqrt(1 - Pow2(h * a)) / CosThetaD[1]);
		float Np = exp(-3.65 * CosPhi[1] - 3.98);

		Ap[1] = Np * Fp * Tp * Backlit;
	}
	// TRT Energy
	if (HairComponents & HAIR_COMPONENT_TRT)
	{
		//float h = 0.75;
		float f = Hair_F(CosThetaD[2] * 0.5);
		float Fp = Pow2(1 - f) * f;
		float3 Tp = pow(BaseColor, 0.8 / CosThetaD[2]);
		float Np = exp(17 * CosPhi[2] - 16.78);

		Ap[2] = Np * Fp * Tp;
	}

	const float3 AverageAp =  float3(dot(Ap[0],0.333f), dot(Ap[1],0.333f), dot(Ap[2],0.333f));
	const float Wp = max(1e-4, AverageAp.x + AverageAp.y + AverageAp.z);
	const float3 LobePdf = AverageAp / Wp;
	if (u1.y <= LobePdf.x)
	{
		p = 0;
	}
	else if (u1.y < (LobePdf.x + LobePdf.y))
	{
		p = 1;
	}
	else
	{
		p = 2;
	}


	// 3. Sample Mp
	const float SinThetaVp = sin(asin(SinThetaV) - Alpha[p]);
	const float CosThetaVp = cos(acos(CosThetaV) - Alpha[p]);

	const float2 CosSinThetaL = SampleMp(u0, B[p], CosThetaVp, SinThetaVp);
	const float PhiL = phiO + dPhiL[p];

	// 4. Weight computation
	// Importance Sampling for Physically-Based Hair Fiber Models - Section 3.2 
	OutWeight = clamp(Ap[p] / Wp, 0, 2);
	OutDirection = float3(CosSinThetaL.y, CosSinThetaL.x * cos(PhiL), CosSinThetaL.x * sin(PhiL));
}



#if 0

float2 Rand1SPPDenoiserInput(uint2 PixelPos, uint FrameIndexMod8)
{
	uint2 Random = Rand3DPCG16(int3(PixelPos, FrameIndexMod8)).xy;
	float2 E = Rand16ToFloat(Random);

	return E;
}

// Example on how to sample the BSDF
// * you need to provide material property such as Roughness/BaseColor, but also ideally the 
// BackLit is often used in card based groom
// * the sampling need 4 random numbers. This example show a simple correlated sample pattern
{
	const uint TileOffset = 17; // Arbitrary, but larger than the reconstruction kernel window
	const float2 RandU = Rand1SPPDenoiserInput(PixelCoord);
	const float2 RandV = Rand1SPPDenoiserInput(PixelCoord + TileOffset);

	const float Specular = 0.5;	
	const float Backlit = 1;	

	float3 OutDirection = float3(0, 0, 1);
	float OutWeight = 1;
	const uint HairComponents = HAIR_COMPONENT_R | HAIR_COMPONENT_TT | HAIR_COMPONENT_TRT;
	SampleHair(Roughness, BaseColor, V, T, RandU, RandV, HairComponents, OutDirection, OutWeight);

}

#endif