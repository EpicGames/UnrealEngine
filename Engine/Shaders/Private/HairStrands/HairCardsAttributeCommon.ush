// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/HairStrandsDefinitions.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

// Vertex factory Flags
#define EHairCardsVFlags_Texture0    0x1
#define EHairCardsVFlags_Texture1    0x2
#define EHairCardsVFlags_Texture2    0x4
#define EHairCardsVFlags_Texture3    0x8
#define EHairCardsVFlags_Texture4    0x10
#define EHairCardsVFlags_Texture5    0x20
#define EHairCardsVFlags_InvertedUV  0x40
#define EHairCardsVFlags_VertexColor 0x80

// Attribute index
#define EHairCardsVFAttribute_Depth        0
#define EHairCardsVFAttribute_Seed         1
#define EHairCardsVFAttribute_CoordU       2
#define EHairCardsVFAttribute_GroupIndex   3
#define EHairCardsVFAttribute_Roughness    4
#define EHairCardsVFAttribute_RootUV       5
#define EHairCardsVFAttribute_Color        6
#define EHairCardsVFAttribute_Tangent      7
#define EHairCardsVFAttribute_Coverage     8
#define EHairCardsVFAttribute_Auxilary     9

#define LAYOUT_VALUES(LayoutIdx, L0, L1, L2, L3) \
	(LayoutIdx == 0 ? L0 :  \
	(LayoutIdx == 1 ? L1 :  \
	(LayoutIdx == 2 ? L2 : L3)))

uint GetHairCardTextureIndex(uint InLayoutIndex, uint InAttribute) 
{ 
	// See GroomAssetCards.cpp, EHairTextureLayout::Layout0/1/2/3 for mapping details. (9 = not present/not available)
	switch (InAttribute)
	{
		case EHairCardsVFAttribute_Depth      : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 1, 1);
		case EHairCardsVFAttribute_Seed       : return LAYOUT_VALUES(InLayoutIndex, 3, 3, 1, 2);
		case EHairCardsVFAttribute_CoordU     : return LAYOUT_VALUES(InLayoutIndex, 3, 3, 0, 0);
		case EHairCardsVFAttribute_GroupIndex : return LAYOUT_VALUES(InLayoutIndex, 9, 4, 9, 1); // Only used for mesh, not for cards
		case EHairCardsVFAttribute_Roughness  : return LAYOUT_VALUES(InLayoutIndex, 9, 9, 9, 9);
		case EHairCardsVFAttribute_RootUV     : return LAYOUT_VALUES(InLayoutIndex, 9, 3, 9, 2);
		case EHairCardsVFAttribute_Color      : return LAYOUT_VALUES(InLayoutIndex, 9, 4, 9, 1);
		case EHairCardsVFAttribute_Tangent    : return LAYOUT_VALUES(InLayoutIndex, 2, 2, 0, 0);
		case EHairCardsVFAttribute_Coverage   : return LAYOUT_VALUES(InLayoutIndex, 1, 1, 1, 2);
		case EHairCardsVFAttribute_Auxilary   : return LAYOUT_VALUES(InLayoutIndex, 5, 5, 9, 9); 
	}
	return 0;
}

bool IsHairCardTextureIndexValid(uint InLayoutIndex, uint InAttribute)
{
	return GetHairCardTextureIndex(InLayoutIndex, InAttribute) < HAIR_CARDS_MAX_TEXTURE_COUNT;
}

uint GetHairCardChannelIndex(uint InLayoutIndex, uint InAttribute) 
{ 
	// See GroomAssetCards.cpp, EHairTextureLayout::Layout0/1/2/3 for mapping details
	switch (InAttribute)
	{
		case EHairCardsVFAttribute_Depth      : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 1, 2);
		case EHairCardsVFAttribute_Seed       : return LAYOUT_VALUES(InLayoutIndex, 3, 3, 2, 2);
		case EHairCardsVFAttribute_CoordU     : return LAYOUT_VALUES(InLayoutIndex, 2, 2, 3, 3);
		case EHairCardsVFAttribute_GroupIndex : return LAYOUT_VALUES(InLayoutIndex, 3, 3, 0, 3);
		case EHairCardsVFAttribute_Roughness  : return LAYOUT_VALUES(InLayoutIndex, 3, 0, 3, 3);
		case EHairCardsVFAttribute_RootUV     : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 0, 0);
		case EHairCardsVFAttribute_Color      : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 0, 0); // For layout 3, Color is only Color.XY The Z channel is overriden by Seed. See GetHairCardsColor()
		case EHairCardsVFAttribute_Tangent    : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 0, 0);
		case EHairCardsVFAttribute_Coverage   : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 0, 3);
		case EHairCardsVFAttribute_Auxilary   : return LAYOUT_VALUES(InLayoutIndex, 0, 0, 0, 0); 
	}
	return 0;
}

#if HAIR_CARD_MESH_FACTORY

#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions for fetching attributes

// If we need custom dynamic layout:
// uint GetHairCardTextureIndex(uint In)	{ return ((HairCards.AttributeTextureIndex) >> (In*3u) & 0x7u); }
// uint GetHairCardChannelIndex(uint In)	{ return ((HairCards.AttributeChannelIndex) >> (In*2u) & 0x3u); }

float2 GetAtlasUV(float2 In) 				{ return float2(In.x, (HairCardsVF.Flags & EHairCardsVFlags_InvertedUV) ? 1 - In.y : In.y); }
bool HasHairCardsVertexColor() 				{ return (HairCardsVF.Flags & EHairCardsVFlags_VertexColor); }

bool HasTextureBound(uint In)
{
	// HairCardsVF.Flags stores InverttUV on the first bit, texture bound after
	const uint TextureIndex = GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In);
	return (TextureIndex < HairCardsVF.TextureCount) && (HairCardsVF.Flags & (1u << TextureIndex)) != 0;
}

// Generic attribute texture fetches
float4 FetchAttributeTexture(uint InTextureIndex, float2 InAtlasUV, float InBias)
{
	float4 Out = 0;
	     if (InTextureIndex == 0) { Out = Texture2DSampleBias(HairCardsVF.Texture0Texture, HairCardsVF.Texture0Sampler, InAtlasUV, InBias); }
	else if (InTextureIndex == 1) { Out = Texture2DSampleBias(HairCardsVF.Texture1Texture, HairCardsVF.Texture1Sampler, InAtlasUV, InBias); }
	else if (InTextureIndex == 2) { Out = Texture2DSampleBias(HairCardsVF.Texture2Texture, HairCardsVF.Texture2Sampler, InAtlasUV, InBias); }
	else if (InTextureIndex == 3) { Out = Texture2DSampleBias(HairCardsVF.Texture3Texture, HairCardsVF.Texture3Sampler, InAtlasUV, InBias); }
	else if (InTextureIndex == 4) { Out = Texture2DSampleBias(HairCardsVF.Texture4Texture, HairCardsVF.Texture4Sampler, InAtlasUV, InBias); }
	else if (InTextureIndex == 5) { Out = Texture2DSampleBias(HairCardsVF.Texture5Texture, HairCardsVF.Texture5Sampler, InAtlasUV, InBias); }
	return Out;
}

float4 FetchAttributeTexture(uint InTextureIndex, float2 InAtlasUV)
{
	float4 Out = 0;
		 if (InTextureIndex == 0) { Out = Texture2DSample(HairCardsVF.Texture0Texture, HairCardsVF.Texture0Sampler, InAtlasUV); }
	else if (InTextureIndex == 1) { Out = Texture2DSample(HairCardsVF.Texture1Texture, HairCardsVF.Texture1Sampler, InAtlasUV); }
	else if (InTextureIndex == 2) { Out = Texture2DSample(HairCardsVF.Texture2Texture, HairCardsVF.Texture2Sampler, InAtlasUV); }
	else if (InTextureIndex == 3) { Out = Texture2DSample(HairCardsVF.Texture3Texture, HairCardsVF.Texture3Sampler, InAtlasUV); }
	else if (InTextureIndex == 4) { Out = Texture2DSample(HairCardsVF.Texture4Texture, HairCardsVF.Texture4Sampler, InAtlasUV); }
	else if (InTextureIndex == 5) { Out = Texture2DSample(HairCardsVF.Texture5Texture, HairCardsVF.Texture5Sampler, InAtlasUV); }
	return Out;
}

// Generic attribute fetch functions
float  FetchAttribute1(uint In, float2 InAtlasUV, float InBias) { const float4 Value = FetchAttributeTexture(GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In), InAtlasUV, InBias); return Value[GetHairCardChannelIndex(HairCardsVF.LayoutIndex, In)];                     }
float  FetchAttribute1(uint In, float2 InAtlasUV)               { const float4 Value = FetchAttributeTexture(GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In), InAtlasUV);         return Value[GetHairCardChannelIndex(HairCardsVF.LayoutIndex, In)];                     }
float2 FetchAttribute2(uint In, float2 InAtlasUV)               { const float4 Value = FetchAttributeTexture(GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In), InAtlasUV);         return GetHairCardChannelIndex(HairCardsVF.LayoutIndex, In) == 0 ? Value.xy : Value.zw; } // float2 alignemnt is forced onto .xy or .zw
float3 FetchAttribute3(uint In, float2 InAtlasUV)               { const float4 Value = FetchAttributeTexture(GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In), InAtlasUV);         return Value.xyz;                                                                       } // float3 alignemnt forced onto .xyz
float4 FetchAttribute4(uint In, float2 InAtlasUV)               { const float4 Value = FetchAttributeTexture(GetHairCardTextureIndex(HairCardsVF.LayoutIndex, In), InAtlasUV);         return Value;                                                                           }

// Special attribute fetch functions
float  GetDepthOffset (float2 InAtlasUV)                        { float  Out = 0;         if (HasTextureBound(EHairCardsVFAttribute_Depth))		 { Out = FetchAttribute1(EHairCardsVFAttribute_Depth,     InAtlasUV);                          } return Out; }
float  GetSeed        (float2 InAtlasUV)                        { float  Out = 0;         if (HasTextureBound(EHairCardsVFAttribute_Seed))		 { Out = FetchAttribute1(EHairCardsVFAttribute_Seed,      InAtlasUV);                          } return Out; }
float  GetCoordU      (float2 InAtlasUV)                        { float  Out = 0;         if (HasTextureBound(EHairCardsVFAttribute_CoordU))	 { Out = FetchAttribute1(EHairCardsVFAttribute_CoordU,    InAtlasUV);                          } return Out; }
float  GetGroupIndex  (float2 InAtlasUV, float  InDefault)      { float  Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_GroupIndex)) { Out = FetchAttribute1(EHairCardsVFAttribute_GroupIndex,InAtlasUV); Out *= 255.f;            } return Out; }
float  GetRoughness   (float2 InAtlasUV, float  InDefault)      { float  Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_Roughness))	 { Out = FetchAttribute1(EHairCardsVFAttribute_Roughness, InAtlasUV);                          } return Out; }
float  GetCoverage    (float2 InAtlasUV, float  InDefault)      { float  Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_Coverage))	 { Out = FetchAttribute1(EHairCardsVFAttribute_Coverage,  InAtlasUV, HairCardsVF.CoverageBias);} return Out; }
float2 GetRootUV      (float2 InAtlasUV, float2 InDefault)      { float2 Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_RootUV))	 { Out = FetchAttribute2(EHairCardsVFAttribute_RootUV,    InAtlasUV);                          } return Out; }
float3 GetColor       (float2 InAtlasUV, float3 InDefault)      { float3 Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_Color))		 { Out = FetchAttribute3(EHairCardsVFAttribute_Color,     InAtlasUV); Out = Pow2(Out);         } return Out; } // Cheap sRGB -> Linear encoding
float3 GetTangent     (float2 InAtlasUV, float3 InDefault)      { float3 Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_Tangent))	 { Out = FetchAttribute3(EHairCardsVFAttribute_Tangent,   InAtlasUV); Out = Out*2-1.f;         } return Out; }
float4 GetAuxilary    (float2 InAtlasUV, float4 InDefault)      { float4 Out = InDefault; if (HasTextureBound(EHairCardsVFAttribute_Auxilary))	 { Out = FetchAttribute4(EHairCardsVFAttribute_Auxilary,  InAtlasUV);                          } return Out; }

///////////////////////////////////////////////////////////////////////////////////////////////////
// MaterialTemplate interface

float GetHairCardsDepth(float2 InAtlasUV, float InDeviceZ)
{
	const float SceneDepthOffset = GetDepthOffset(InAtlasUV);
	const float SceneDepth = ConvertFromDeviceZ(InDeviceZ);
	return ConvertToDeviceZ(SceneDepth + SceneDepthOffset);
}

float3 GetHairCardsTangent(float2 InAtlasUV, half3x3 TangentToWorld, bool bUseTangentSpace)
{
	const float3 LocalTangent = GetTangent(InAtlasUV, float3(0, 1, 0));
	const float3 WorldTangent = mul(LocalTangent, TangentToWorld);
	return bUseTangentSpace ? LocalTangent : WorldTangent;
}

float GetHairCardsCoverage(float2 InAtlasUV)
{
	return GetCoverage(InAtlasUV, 1.f);
}

float4 GetHairCardsAuxilaryData(float2 InAtlasUV)
{
	return GetAuxilary(InAtlasUV, 0.f);
}

float2 GetHairCardsUV(float2 InAtlasUV)
{
	return float2(GetCoordU(InAtlasUV), 0.5f);
}

float2 GetHairCardsDimensions(float2 InAtlasUV, float HairPrimitiveLength)
{
	return float2(HairPrimitiveLength, 0.01f/*WorldRadius*/);
}

float2 GetHairCardsRootUV(float2 InAtlasUV, float2 InRootUV)
{
	// Invert V to compensate image origin flip
	// Similar to DecodeHairAttribute in HairStrandVertexFactoryCommon.ush
	return GetRootUV(InAtlasUV, float2(InRootUV.x, 1.0 - InRootUV.y));
}

float GetHairCardsSeed(float2 InAtlasUV)
{
	return GetSeed(InAtlasUV);
}

uint3 GetHairCardsClumpID(float2 InAtlasUV)
{
	return 0;
}

float3 GetHairCardsColor(float2 InAtlasUV, float3 InColor)
{
	float3 OutColor = GetColor(InAtlasUV, InColor);
	return HairCardsVF.LayoutIndex == 3 ? float3(OutColor.xy, 0) : OutColor;
}

float GetHairCardsRoughness(float2 InAtlasUV, float InCardRoughness)
{
	//return GetRoughness(InAtlasUV, InCardRoughness);
	return (HairCardsVF.Flags & EHairCardsVFAttribute_Roughness) ? 0 : InCardRoughness;
}

float GetHairCardsAO(float2 InAtlasUV, float InAO)
{
	return 1.0f;
}

float GetHairCardsGroupIndex(float2 InAtlasUV, float InCardGroupIndex)
{
	// * For hair cards,  group index is stored onto vertices
	// * For hair meshes, group index is stored within a texture
	return GetGroupIndex(InAtlasUV, InCardGroupIndex);
}
////////////////////////////////////////////////////////////////////////////////

#endif // HAIR_CARD_MESH_FACTORY