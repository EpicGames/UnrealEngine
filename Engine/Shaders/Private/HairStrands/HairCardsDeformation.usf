// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsPack.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsBindingCommon.ush"

#define DEBUG_ENABLE 0

#if DEBUG_ENABLE
#include "../ShaderPrint.ush"
#endif

#if SHADER_DEFORM_CARDS
float4x4 LocalToWorld;

uint CardLODIndex;
uint InstanceRegisteredIndex;
uint CardsVertexCount;
uint GuideVertexCount;
float3 GuideRestPositionOffset;
StructuredBuffer<float4> GuideDeformedPositionOffsetBuffer;

ByteAddressBuffer GuideRestPositionBuffer;
ByteAddressBuffer GuideDeformedPositionBuffer;
Buffer<float4> CardsRestPositionBuffer;
ByteAddressBuffer CardsInterpolationBuffer;

RWBuffer<float4> CardsDeformedPositionBuffer; 

Buffer<float4> CardsRestTangentBuffer;
RWBuffer<float4> CardsDeformedTangentBuffer;

// Guides == Strands going through the middle of the card geometry
Buffer<float4>  TriangleRestPositionBuffer;

Buffer<float4>  TriangleDeformedPositionBuffer;

Buffer<uint>		GuideRootBarycentricBuffer;
ByteAddressBuffer   GuideVertexToRootIndexBuffer;
Buffer<uint>    	GuideRootToUniqueTriangleIndexBuffer;

#if PERMUTATION_DYNAMIC_GEOMETRY == 1
float3 ComputeDynamicGeometryOffset(
	uint GuideIndex,
	float GuideVertexWeight,
	FHairMeshBasis RestBasis,
	FHairMeshBasis DeformedBasis,
	float3 GuideDeformedPositionOffset)
{
	const float3 RestGuidePoint = ReadHairControlPointPosition(GuideRestPositionBuffer, GuideIndex, GuideRestPositionOffset);
	const float3 LocalRestGuidePoint = ToTriangle(RestGuidePoint, RestBasis);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(GuideDeformedPositionBuffer, GuideIndex, GuideDeformedPositionOffset);
	const float3 LocalDeformedGuidePoint = ToTriangle(DeformedGuidePoint, DeformedBasis);

	return (LocalDeformedGuidePoint - LocalRestGuidePoint) * GuideVertexWeight;
}
#endif

void TransformTangent(
	uint InVertexId, 
	uint InOffset,
	FHairMeshBasis InRestBasis,
	FHairMeshBasis InDeformedBasis)
{
	float4 Rest		= CardsRestTangentBuffer[InVertexId * 2 + InOffset];
	float3 Local	= VectorToTriangle(Rest.xyz, InRestBasis);
	float3 Deformed = VectorToWorld(Local, InDeformedBasis);
	CardsDeformedTangentBuffer[InVertexId * 2 + InOffset] = float4(Deformed, Rest.w);
}

void CopyTangent(uint InVertexId, uint InOffset)
{
	CardsDeformedTangentBuffer[InVertexId * 2 + InOffset] = CardsRestTangentBuffer[InVertexId * 2 + InOffset];
}

const FHairMeshBasis ConstructBasis(float3 GuideP0, float3 GuideP1, float3 P)
{
	FHairMeshBasis Out;

	const float3 V0 = P;
	const float3 V1 = GuideP0;
	const float3 V2 = GuideP1;

	const float3 E0 = (V1 - V0).xyz;
	const float3 E1 = (V2 - V0).xyz;

	// Build orthonormal frame
	Out.BaseO = P;  //(V0 * B.x + V1 * B.y + V2 * B.z).xyz;
	Out.BaseN = normalize(cross(E1, E0));
	Out.BaseT = normalize(E0);
	Out.BaseB = normalize(cross(Out.BaseT, Out.BaseN));
	return Out;
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint VertexId = DispatchThreadId;
	if (VertexId >= CardsVertexCount)
		return;

	const float GuideRadius = 1;
	const float GuideRootScale = 1;
	const float GuideTipScale = 1;
	const float3 GuideDeformedPositionOffset = ReadCardPositionOffset(GuideDeformedPositionOffsetBuffer, InstanceRegisteredIndex, CardLODIndex);

#if PERMUTATION_DYNAMIC_GEOMETRY == 0

	const FCardsGuideData GuideData = ReadCardGuideData(CardsInterpolationBuffer, VertexId);
	const uint GuidePointIndex0 = GuideData.VertexIndex;
	const uint GuidePointIndex1 = min(GuidePointIndex0+1, GuideVertexCount-1);

	const FHairControlPoint RestGuideP0 = ReadHairControlPoint(
		GuideRestPositionBuffer, 
		GuidePointIndex0,
		GuideRestPositionOffset,
		GuideRadius,
		GuideRootScale,
		GuideTipScale);

	const FHairControlPoint RestGuideP1 = ReadHairControlPoint(
		GuideRestPositionBuffer,
		GuidePointIndex1,
		GuideRestPositionOffset,
		GuideRadius,
		GuideRootScale,
		GuideTipScale);

	const FHairControlPoint DeformedGuideP0 = ReadHairControlPoint(
		GuideDeformedPositionBuffer,
		GuidePointIndex0,
		GuideDeformedPositionOffset,
		GuideRadius,
		GuideRootScale,
		GuideTipScale);

	const FHairControlPoint DeformedGuideP1 = ReadHairControlPoint(
		GuideDeformedPositionBuffer,
		GuidePointIndex1,
		GuideDeformedPositionOffset,
		GuideRadius,
		GuideRootScale,
		GuideTipScale);

	const float3 RestGuidePosition		= lerp(RestGuideP0.Position, RestGuideP1.Position, GuideData.VertexLerp);
	const float3 DeformedGuidePosition	= lerp(DeformedGuideP0.Position, DeformedGuideP1.Position, GuideData.VertexLerp);

	const float3 DeformationOffset = DeformedGuidePosition - RestGuidePosition;
	const float4 RestPosition = CardsRestPositionBuffer[VertexId];
	const float4 DeformedPosition = float4(RestPosition.xyz + DeformationOffset, RestPosition.w);
	CardsDeformedPositionBuffer[VertexId] = DeformedPosition;

	// Recompute the vertex's normal
	// Construct a local basis from the 2 guides points and the position to be deformed. 
	const FHairMeshBasis RestBasis = ConstructBasis(RestGuideP0.Position, RestGuideP1.Position, RestPosition.xyz);
	const FHairMeshBasis DeformedBasis = ConstructBasis(DeformedGuideP0.Position, DeformedGuideP1.Position, DeformedPosition.xyz);
	TransformTangent(VertexId, 0, RestBasis, DeformedBasis);
	TransformTangent(VertexId, 1, RestBasis, DeformedBasis);

#elif PERMUTATION_DYNAMIC_GEOMETRY == 1
	// Compute the deformation offset in triangle local space

	const FCardsGuideData GuideData = ReadCardGuideData(CardsInterpolationBuffer, VertexId);
	const uint GuidePointIndex0 = GuideData.VertexIndex;
	const uint GuidePointIndex1 = min(GuidePointIndex0 + 1, GuideVertexCount - 1);

	const uint GuideCurveIndex = ReadHairPointToCurveIndex(GuideVertexToRootIndexBuffer, GuidePointIndex0);
	const uint TriangleIndex = GuideRootToUniqueTriangleIndexBuffer[GuideCurveIndex];

	const float3 RootBarycentric = UnpackBarycentrics(GuideRootBarycentricBuffer[GuideCurveIndex]);

	const FHairMeshBasis RestBasis = GetHairMeshBasis(TriangleIndex, TriangleRestPositionBuffer, RootBarycentric);
	const FHairMeshBasis DeformedBasis = GetHairMeshBasis(TriangleIndex, TriangleDeformedPositionBuffer, RootBarycentric);

	const float3 Offset0 = ComputeDynamicGeometryOffset(GuidePointIndex0, 1, RestBasis, DeformedBasis, GuideDeformedPositionOffset);
	const float3 Offset1 = ComputeDynamicGeometryOffset(GuidePointIndex1, 1, RestBasis, DeformedBasis, GuideDeformedPositionOffset);
	const float3 CurrOffset = VectorToWorld(lerp(Offset0, Offset1, GuideData.VertexLerp), DeformedBasis);

	// Transform hair from rest post to deformed pose, based triangle deformation + simulation offset (optional depending of the simulation is actual running or not
	const float4 RestControlPoint = CardsRestPositionBuffer[VertexId];
	float3 ControlPoint = RestControlPoint.xyz;
	ControlPoint = ToTriangle(ControlPoint, RestBasis);
	ControlPoint = ToWorld(ControlPoint, DeformedBasis) + CurrOffset;

	#if DEBUG_ENABLE
	{
		float3 WP0 = mul(float4(DeformedBasis.BaseO, 1), LocalToWorld).xyz;
		float3 WP1 = mul(float4(DeformedBasis.BaseO + DeformedBasis.BaseT, 1), LocalToWorld).xyz;
		float3 WP2 = mul(float4(DeformedBasis.BaseO + DeformedBasis.BaseB, 1), LocalToWorld).xyz;
		AddLineTriangle(WP0, WP1, WP2, ColorOrange);
	}
	#endif

	CardsDeformedPositionBuffer[VertexId] = float4(ControlPoint, RestControlPoint.w);

	// Transform tangent
	TransformTangent(VertexId, 0, RestBasis, DeformedBasis);
	TransformTangent(VertexId, 1, RestBasis, DeformedBasis);
#endif // PERMUTATION_DYNAMIC_GEOMETRY
}
#endif // SHADER_DEFORM_CARDS