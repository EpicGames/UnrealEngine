// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShaderPrint.ush"
#include "../MonteCarlo.ush"

#include "HairStrandsVisibilityCommon.ush"
#include "HairCardsTextureCommon.ush"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_VERTEX_RECT || SHADER_PIXEL_RECT

uint SampleCount;

int2 Atlas_Resolution;
int2 Atlas_RectOffset;
int2 Atlas_RectResolution;

int2 Tile_Coord;
int2 Tile_Count;
int2 Tile_Resolution;

float3 Raster_MinBound;
float3 Raster_MaxBound;

float3 Raster_AxisX; // Not normalized, contains AABB (full) length
float3 Raster_AxisY; // Not normalized, contains AABB (full) length
float3 Raster_AxisZ; // Not normalized, contains AABB (full) length

// Offset to the first strands vertex
// Count of all vertex belonging to this cluster
uint Curve_VertexOffset;
uint Curve_VertexCount;
uint Curve_TotalVertexCount;

Buffer<float4> Curve_PositionBuffer;
Buffer<float4> Curve_AttributeBuffer;

float GetPixelWorldRadius()
{
	return select(IsOrthoProjection(), 0.5f, 0.5f * max(length(Raster_AxisX) / float(Atlas_RectResolution.x), length(Raster_AxisY) / float(Atlas_RectResolution.y)));
}
#endif

#if SHADER_VERTEX_RECT

#include "HairStrandsVertexFactoryCommon.ush"

void MainVS(
	uint VertexId : SV_VertexID,
	out float4 OutPosition : SV_Position,
	nointerpolation out uint OutVertexId : SEG_ID)
{
	const int VertexIndex = Curve_VertexOffset + (VertexId / 6);
	OutVertexId = VertexIndex;

	int V0_1 = max(VertexIndex - 1, 0);
	int V0_0 = max(VertexIndex + 0, 0);
	int V1_0 = max(VertexIndex + 1, 0);
	int V1_1 = max(VertexIndex + 2, 0);
	V0_1 = min(V0_1, Curve_TotalVertexCount - 1);
	V0_0 = min(V0_0, Curve_TotalVertexCount - 1);
	V1_0 = min(V1_0, Curve_TotalVertexCount - 1);
	V1_1 = min(V1_1, Curve_TotalVertexCount - 1);

	const float4 Data0_1 = Curve_PositionBuffer[V0_1];
	const float4 Data0   = Curve_PositionBuffer[V0_0];
	const float4 Data1   = Curve_PositionBuffer[V1_0];
	const float4 Data1_1 = Curve_PositionBuffer[V1_1];

	const float3 CP0_1 = Data0_1.xyz;
	const float3 CP0   = Data0.xyz;
	const float3 CP1   = Data1.xyz;
	const float3 CP1_1 = Data1_1.xyz;
	const float R0 = Data0.w;
	const float R1 = Data1.w;


	//				CP0    - - - - - -> B
	// 0 ---2  4          |
	// |   /  / |         |
	// |  /  /  |         |
	// | /  /   |         |
	// 1   3----5   CP1   v T
	const uint LocalVertexId = (VertexId % 6);
	const bool bBottom = (LocalVertexId % 2);
	const bool bLeft   = LocalVertexId == 0 || LocalVertexId == 1 || LocalVertexId == 3;

	float3 CP  = 0;
	float R    = 0;
	float Sign = bLeft ? -1 : 1;
	if (bBottom)
	{
		CP = CP0;
		R = R0;
	}
	else
	{
		CP = CP1;
		R = R1;
	}

	float3 CP_T = 0;
	if (bBottom)
	{
		const float3 CP_T0 = Data0_1.w > 0 ? CP0.xyz - CP0_1.xyz : 0;
		const float3 CP_T1 = CP1.xyz - CP0.xyz;
		CP_T = normalize(CP_T1 + CP_T0);
	}
	else
	{
		const float3 CP_T0 = CP1.xyz - CP0.xyz;
		const float3 CP_T1 = Data1.w > 0 ? CP1_1.xyz - CP1.xyz : 0;
		CP_T = normalize(CP_T1 + CP_T0);
	}

	float2 UV  = 0;
	UV.x = bLeft ? 0.f : 1.f;
	UV.y = bBottom ? 0.f : 1.f;

	const float3 CP_B = normalize(cross(CP_T, Raster_AxisZ));

	// Clamp poly strip to be at least 2 pixels wide
	const float PixelRadius = GetPixelWorldRadius();
	const float RadiusScale = 2;
	const float RasterRadius = max(R, PixelRadius * RadiusScale);
	const float3 WorldPosition = CP.xyz + CP_B * RasterRadius * Sign + CP_T * RasterRadius * Sign;

	const float3 NormPos = (WorldPosition - Raster_MinBound) / (Raster_MaxBound - Raster_MinBound);
	const float3 AxisX = normalize(Raster_AxisX);
	const float3 AxisY = normalize(Raster_AxisY);
	const float3 AxisZ = normalize(Raster_AxisZ);

	const float SignX = dot(AxisX, float3(1, 1, 1)) > 0 ? 1 : -1;
	const float SignY = dot(AxisY, float3(1, 1, 1)) > 0 ? 1 : -1;
	const float SignZ = dot(AxisZ, float3(1, 1, 1)) > 0 ? 1 : -1;

	float3 NDCPos = 0;
	NDCPos.x = dot(AxisX, NormPos) * 2 + (SignX>0 ? -1 :  1);
	NDCPos.y = dot(AxisY, NormPos) * 2 + (SignY>0 ? -1 :  1);
	NDCPos.z = dot(AxisZ, NormPos)     * (SignZ>0 ?  1 : -1);

	// Cull invalid segment
	OutPosition	 = float4(NDCPos, 1);
	if (R0 == 0)
	{
		OutPosition = float4(INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT, 1);
	}
}

#endif // VERTEXSHADER


/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_PIXEL_RECT

#include "HairStrandsVertexFactoryCommon.ush"

#define DEBUG_ENABLE 0

// Two output type:
// 0 = Depth/Tangent/Attributes
// 1 = Coverage
void MainPS(
	in float4  SvPosition : SV_Position,
	nointerpolation in uint VertexId : SEG_ID,
#if PERMUTATION_OUTPUT == 0
	out float4 OutDepth : SV_Target0,
	out float4 OutTangent : SV_Target1,
	out float4 OutAttribute : SV_Target2
#else
	out float4 OutCoverage : SV_Target0
#endif
)
{
	float2 InRectUV = float2(SvPosition.xy - Atlas_RectOffset) / float2(Atlas_RectResolution);
	InRectUV.y = 1 - InRectUV.y;

#if PERMUTATION_OUTPUT == 0
	float4 OutCoverage = 0;
#else
	float4 OutDepth = 0;
	float4 OutTangent = 0;
	float4 OutAttribute = 0;
#endif
	OutCoverage = 0;
	OutDepth = 0;
	OutTangent = float4(InRectUV, 0, 0);
	OutAttribute = 0;

	const float3 Center = (Raster_MaxBound + Raster_MinBound) * 0.5f;
	const float3 CardWorldPos =
		(InRectUV.x * 2 - 1) * Raster_AxisX * 0.5f +
		(InRectUV.y * 2 - 1) * Raster_AxisY * 0.5f +
		-0.5f * Raster_AxisZ +
		Center;

	const int2 PixelCoord = int2(SvPosition.xy);

	#if DEBUG_ENABLE
	bool bDebugEnable = VertexId <= 10; // false;// all(PixelCoord == int2(1600, 10));
	if (bDebugEnable)
	{
		const float4 Color = float4(1, 1, 0, 1);
		AddAABBWS(Raster_MinBound, Raster_MaxBound, Color);
	}
	#endif 

	const float3 RayO = CardWorldPos;
	const float3 RayD = normalize(Raster_AxisZ);

	const float MaxDistance = 10;
	#if DEBUG_ENABLE
	if (bDebugEnable)
	{
		const float4 Color = float4(1, 0, 0, 1);
		AddLineWS(RayO, RayO + RayD * MaxDistance, Color, Color);
	}
	#endif

	const float FarDistance = 100000;
	float ClosestDepth = FarDistance;
	float3 ClosestTangent = 0;
	float4 ClosestAttribute = 0;

	//for (uint VertexIt = 0; VertexIt < Curve_VertexCount-1; ++VertexIt)
	{
		const uint VertexIndex0 = clamp(VertexId,   0, Curve_TotalVertexCount - 1);
		const uint VertexIndex1 = clamp(VertexId+1, 0, Curve_TotalVertexCount - 1);
		const float4 Data0 = Curve_PositionBuffer[VertexIndex0];
		const float4 Data1 = Curve_PositionBuffer[VertexIndex1];

		const float4 Attributes0 = Curve_AttributeBuffer[VertexIndex0];
		const float4 Attributes1 = Curve_AttributeBuffer[VertexIndex1];

		const float3 CP0 = Data0.xyz;
		const float3 CP1 = Data1.xyz;

		const float R0 = Data0.w;
		const float R1 = Data1.w;

		// Invalid selction, i.e., a segment connecting the end of a strands to the beginning of another strands.
		if (Data0.w == 0)
			discard;

		#if DEBUG_ENABLE
		if (bDebugEnable)
		{
			const float4 Color = float4(0, 1, 0, 1);
			AddLineWS(CP0, CP1, Color, Color);
		}
		#endif

		const float PixelRadius = GetPixelWorldRadius(); 

		const float3 WorldTangent = normalize(CP1 - CP0);
		const float3 LocalTangent = float3(
			dot(WorldTangent, Raster_AxisX),
			dot(WorldTangent, Raster_AxisY),
			dot(WorldTangent, -Raster_AxisZ));

		for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			const float2 U = Hammersley(SampleIt, SampleCount, 0) * 2 - 1;

			const float3 Jitter = 
				U.x * (0.5f * Raster_AxisX / float(Atlas_RectResolution.x)) +
				U.y * (0.5f * Raster_AxisY / float(Atlas_RectResolution.y));

			const float3 RayP0 = RayO + Jitter;
			const float3 RayP1 = RayO + Jitter + RayD * MaxDistance;
			#if DEBUG_ENABLE
			if (bDebugEnable)
			{
				const float4 Color = float4(1, 0, 0, 1);
				AddLineWS(RayP0, RayP1, Color, Color);
			}
			#endif
			// Scale radius in order to avoid having 0 coverage when hair are too thin
			float VertexU = 0;
			const float RadiusScaleFactor = 1.5f;
			const float Distance = Intersection(CP0, CP1, RayP0, RayP1, R0, VertexU) * MaxDistance;
			if (Distance > 0 && Distance < ClosestDepth)
			{
				OutCoverage		+= saturate(R0* RadiusScaleFactor / PixelRadius);
				ClosestTangent	= LocalTangent;
				ClosestDepth	= saturate(Distance / length(RayP1 - RayP0));
				ClosestAttribute= lerp(Attributes0, Attributes1, VertexU);
			}
		}
	}

	OutCoverage /= SampleCount;
	const float3 NColor = (ClosestTangent + 1) * 0.5f;
	OutTangent = float4(NColor, 0);
	OutDepth = ClosestDepth < FarDistance ? ClosestDepth : 0;
	OutAttribute = ClosestAttribute;

	OutCoverage.w	= 1;
	OutTangent.w	= 1;
	OutDepth.w		= 1;
	OutAttribute.w	= 1;


#if PERMUTATION_OUTPUT == 0
	if (OutCoverage.x <= 0)
	{
		discard;
	}
#endif
}

#endif // SHADER_PIXEL_RECT
