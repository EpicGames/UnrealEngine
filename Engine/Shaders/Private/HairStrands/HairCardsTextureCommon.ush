// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Tracing

float2 LineLineIntersect(float3 p1, float3 p2, float3 p3, float3 p4)
{
	float3 p13, p43, p21;
	float d1343, d4321, d1321, d4343, d2121;
	float numer, denom;
	float mua;

	p13 = p1 - p3;
	p43 = p4 - p3;
	p21 = p2 - p1;

	d1343 = dot(p13, p43);
	d4321 = dot(p43, p21);
	d1321 = dot(p13, p21);
	d4343 = dot(p43, p43);
	d2121 = dot(p21, p21);

	denom = d2121 * d4343 - d4321 * d4321;
	//if (abs(denom) < 0.0000001)//almost impossible: view ray perfectly aligned with line direction (and it would only affect one pixel)
	//	return float2(0.0,0.0);
	numer = d1343 * d4321 - d1321 * d4343;

	mua = numer / denom;
	mua = clamp(mua, 0.0, 1.0);
	return float2(mua, (d1343 + d4321 * mua) / d4343); //return (mua,mub)
}

float Intersection(float3 InP0, float3 InP1, float3 RayP0, float3 RayP1, float Radius, inout float OutU)
{
	float3 P1 = InP0;
	float3 P2 = InP1;
	float3 P3 = RayP0;
	float3 P4 = RayP1;

	//compute the two closest points on the volumetric line and current view ray
	float2 muab = LineLineIntersect(P1, P2, P3, P4);

	//pa and pb, the two closest points
	float3 pa = P1 + muab.x * (P2 - P1);
	float3 pb = P3 + muab.y * (P4 - P3);

	//texture sample coordinate
	const float sample = length(pa - pb) / Radius;
	OutU = muab.x;

	return sample < 1 ? muab.y : -1;
}