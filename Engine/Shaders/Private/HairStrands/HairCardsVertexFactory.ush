// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StrandHairFactory.usf
=============================================================================*/
#include "../VertexFactoryCommon.ush"
#include "../LocalVertexFactoryCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairCardsAttributeCommon.ush"

#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"

#if RAYHITGROUPSHADER
#include "../RayTracing/RayTracingCommon.ush"
#include "../RayTracing/RayTracingHitGroupCommon.ush"
#endif

#define USE_HAIR_COMPLEX_TRANSMITTANCE 1

#if MANUAL_VERTEX_FETCH
	Buffer<float4> VertexFetch_InstanceOriginBuffer;
	Buffer<float4> VertexFetch_InstanceTransformBuffer;
#endif

////////////////////////////////////////////////////////////////////////////////
// HAIR_CARD_MESH_FACTORY
// This is set by compilation environment of the vertex factory
///////////////////////////////////////////////////////////////////////////////

/**
 * Per-vertex inputs from bound vertex buffers
 */

struct FVertexFactoryInput
{
#if MANUAL_VERTEX_FETCH
	uint VertexId : SV_VertexID;
#else
	float4 Position      : ATTRIBUTE0; 
	float4 PrevPosition  : ATTRIBUTE1;

	// TangentZ.w contains sign of tangent basis determinant
  #if METAL_ES3_1_PROFILE
	float3	TangentX : ATTRIBUTE2;
	float4	TangentZ : ATTRIBUTE3;
  #else
	half3	TangentX : ATTRIBUTE2;
	half4	TangentZ : ATTRIBUTE3;
  #endif
	float4 Materials : ATTRIBUTE4;
	float4 VertexColors : ATTRIBUTE5;
	float4 UVs : ATTRIBUTE6; // RootUV/AtlasUV

  #if NUM_MATERIAL_TEXCOORDS_VERTEX
	// These used to be packed texcoord arrays, but these cause problems with alighnment on some Vulkan drivers
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 1
	float4	TexCoords0 : ATTRIBUTE7;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
	float4	TexCoords1 : ATTRIBUTE8;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 5
	float4	TexCoords2 : ATTRIBUTE9;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 7
	float4	TexCoords3 : ATTRIBUTE10;
	#endif
  #endif
#endif

	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FTriangleBaseAttributes TriangleAttributes = LoadTriangleBaseAttributes(TriangleIndex);

	FVertexFactoryInput Input = (FVertexFactoryInput)0;
#if MANUAL_VERTEX_FETCH
	Input.VertexId = TriangleAttributes.Indices[VertexIndex];
#endif

	// Note: GetInstanceUserData() stores the GPU-Scene instance ID
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	return Input;
}
#endif

/** 
 * Caches intermediates that would otherwise have to be computed multiple times.  Avoids relying on the compiler to optimize out redundant operations.
 */
struct FVertexFactoryIntermediates
{
	half3x3 TangentToLocal;
	half3x3 TangentToWorld;
	half TangentToWorldSign;
	half4 VertexColors;
	float2 HairPrimitiveLength;
	float4 HairPrimitiveMaterial;
	float  HairPrimitiveGroupIndex;

	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

uint GetHairVertexFetchId(FVertexFactoryInput Input)
{
#if MANUAL_VERTEX_FETCH
	return min(Input.VertexId, HairCardsVF.MaxVertexCount - 1u);
#else
	return 0u;
#endif
}

struct FHairCardsPoint
{
	float3 Position;
	float  CardLength;
	float  GroupIndex;
	float4 Materials;
	half4  VertexColors;
};

FHairCardsPoint GetVertexPosition(FVertexFactoryInput Input)
{
	half4 VertexColors = 0;
#if MANUAL_VERTEX_FETCH
	const uint VertexFetchId = GetHairVertexFetchId(Input);
	const float4 Data = HairCardsVF.PositionBuffer[VertexFetchId];
	const float4 Materials = HairCardsVF.MaterialsBuffer[VertexFetchId];
	if (HasHairCardsVertexColor())
	{
		VertexColors = HairCardsVF.VertexColorsBuffer[VertexFetchId] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
	}
#else
	const float4 Data = Input.Position;
	const float4 Materials = Input.Materials;
	if (HasHairCardsVertexColor())
	{
		VertexColors = Input.VertexColors FCOLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
	}
#endif

	const uint EncodedW = asuint(Data.w);

	FHairCardsPoint Out;
	Out.Position	= Data.xyz;
	Out.CardLength	= f16tof32(EncodedW & 0xFFFF);
	Out.GroupIndex	= (EncodedW>>16u);
	Out.Materials	= float4(Materials.xyz*Materials.xyz, Materials.w); // Decoding (cheap) sRGB -> linear color
	Out.VertexColors= VertexColors;
	return Out;
}

float3 GetVertexPreviousPosition(FVertexFactoryInput Input)
{
#if MANUAL_VERTEX_FETCH
	const uint VertexFetchId = GetHairVertexFetchId(Input);
	const float4 Data = HairCardsVF.PreviousPositionBuffer[VertexFetchId];
#else
	const float4 Data = Input.PrevPosition;
#endif
	return Data.xyz;
}

// Segment UV coord of an hair segment. This is different from the UV coord of the hair strands
float2 GetSegmentUVCoord(FVertexFactoryInput Input)
{
	//FVertexInfo VertexInfo = GetVertexInfo(Input);
	//const float VCoord = VertexInfo.IsLeft ? 0.0f : 1.0f;
	//const float UCoord = VertexInfo.IsTip  ? 1.0f : 0.0f;
	//return float2(UCoord, VCoord);
	return 0; //TODO
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

	half3 TangentToWorld0 = GetTangentToWorld0(Interpolants).xyz;
	half4 TangentToWorld2 = GetTangentToWorld2(Interpolants);
	Result.UnMirrored = TangentToWorld2.w;

	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

	Result.TwoSidedSign = 1;
	Result.PrimitiveId = ToScalarMemory(GetPrimitiveId(Interpolants));
	Result.HairPrimitiveUV = float2(Interpolants.HairPrimitiveUV);
	Result.HairPrimitiveRootUV = float2(Interpolants.HairPrimitiveRootUV);
	Result.HairPrimitiveLength = Interpolants.HairPrimitiveLength;
	Result.HairPrimitiveGroupIndex = Interpolants.HairPrimitiveGroupIndex;
	Result.HairPrimitiveMaterial = Interpolants.HairPrimitiveMaterial;
	Result.VertexColor = GetColor(Interpolants);

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

	return Result;
}

half3x3 CalcTangentToWorldNoScale(FVertexFactoryIntermediates Intermediates, half3x3 TangentToLocal)
{
	half3x3 LocalToWorld = DFToFloat3x3(GetPrimitiveData(Intermediates).LocalToWorld);
	half3 InvScale = GetPrimitiveData(Intermediates).InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;
	return mul(TangentToLocal, LocalToWorld); 
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

void GetRootUVsAtlasUVs(FVertexFactoryInput Input, inout float2 OutAtlasUV, inout float2 OutRootUV)
{
#if MANUAL_VERTEX_FETCH
	const uint NumFetchUVs = DivideAndRoundUp2(HairCardsVF.NumUVs);
	const uint UVFetchIndex = Input.VertexId * NumFetchUVs;
	const float4 UVs = HairCardsVF.UVsBuffer[UVFetchIndex];
	OutRootUV  = UVs.xy;
	OutAtlasUV = UVs.zw;
#else
	OutRootUV  = Input.UVs.xy;
	OutAtlasUV = Input.UVs.zw;
#endif
}

#if NUM_MATERIAL_TEXCOORDS_VERTEX
void GetTexCoords(FVertexFactoryInput Input, inout float2 OutTexCoords[NUM_MATERIAL_TEXCOORDS_VERTEX])
{
#if MANUAL_VERTEX_FETCH
	const uint NumFetchUVs = DivideAndRoundUp2(HairCardsVF.NumUVs);
	UNROLL
	for (uint CoordIndex = 0; CoordIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; ++CoordIndex)
	{
		OutTexCoords[CoordIndex] = 0;
		const uint ReadCoordIndex = CoordIndex + 1; // UVs[0] == RootUV, which is skipped 
		if (ReadCoordIndex < HairCardsVF.NumUVs)
		{
			const uint FetchIndex = Input.VertexId * NumFetchUVs + (ReadCoordIndex >> 1u);
			const float4 PackedUVs = HairCardsVF.UVsBuffer[FetchIndex];
			OutTexCoords[CoordIndex] = (CoordIndex & 1) == 0 ? PackedUVs.zw : PackedUVs.xy; //.xy/.zw are inversed, as TexCoord are shifted by 1 due to UVs[0] == RootUV, which is skipped 
		}
	}
#else // MANUAL_VERTEX_FETCH
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
	OutTexCoords[0] = Input.TexCoords0.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
	OutTexCoords[1] = Input.TexCoords0.zw;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
	OutTexCoords[2] = Input.TexCoords1.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
	OutTexCoords[3] = Input.TexCoords1.zw;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4
	OutTexCoords[4] = Input.TexCoords2.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5
	OutTexCoords[5] = Input.TexCoords2.zw;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 6
	OutTexCoords[6] = Input.TexCoords3.xy;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 7
	OutTexCoords[7] = Input.TexCoords3.zw;
	#endif
#endif  //MANUAL_VERTEX_FETCH 
}
#endif // NUM_MATERIAL_TEXCOORDS_VERTEX

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive
	
	Result.VertexColor = Intermediates.VertexColors;

	// does not handle instancing!
	Result.TangentToWorld = Intermediates.TangentToWorld;
	
	Result.PrevFrameLocalToWorld = GetPrimitiveData(Intermediates).PreviousLocalToWorld;
	Result.PreSkinnedPosition = GetVertexPreviousPosition(Input);
	Result.PreSkinnedNormal = TangentToLocal[2];

	Result.LWCData = MakeMaterialLWCData(Result);

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	GetTexCoords(Input, Result.TexCoords);
#endif

	return Result;
}

float4 CalcWorldPosition(float3 Position, FDFMatrix LocalToWorld)
{
	return TransformLocalToTranslatedWorld(Position, LocalToWorld);
}

half3x3 CalcTangentToLocal(FVertexFactoryInput Input, inout float TangentSign)
{
#if MANUAL_VERTEX_FETCH
	const uint VertexFetchId = GetHairVertexFetchId(Input);
	const half3 TangentInputX = HairCardsVF.NormalsBuffer[VertexFetchId * 2 + 0].xyz;
	const half4 TangentInputZ = HairCardsVF.NormalsBuffer[VertexFetchId * 2 + 1];
#else
	const half3 TangentInputX = Input.TangentX.xyz;
	const half4 TangentInputZ = Input.TangentZ;
#endif

	half3 TangentX = TangentBias(TangentInputX);
	half4 TangentZ = TangentBias(TangentInputZ);
	TangentSign = TangentZ.w;
	
	// derive the binormal by getting the cross product of the normal and tangent
	half3 TangentY = cross(TangentZ.xyz, TangentX) * TangentZ.w;
	
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	half3x3 Result;
	Result[0] = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	Result[1] = TangentY;
	Result[2] = TangentZ.xyz;

	return Result;
}

half3x3 CalcTangentToWorld(FVertexFactoryIntermediates Intermediates, half3x3 TangentToLocal)
{
	half3x3 TangentToWorld = CalcTangentToWorldNoScale(Intermediates, TangentToLocal);
	return TangentToWorld;
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FHairCardsPoint CardsPoint = GetVertexPosition(Input);

	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData					= VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.HairPrimitiveLength		= CardsPoint.CardLength; 
	Intermediates.HairPrimitiveGroupIndex	= CardsPoint.GroupIndex;
	Intermediates.HairPrimitiveMaterial		= CardsPoint.Materials;
	Intermediates.VertexColors				= CardsPoint.VertexColors;

	float TangentSign;
	Intermediates.TangentToLocal = CalcTangentToLocal(Input, TangentSign);
	Intermediates.TangentToWorld = CalcTangentToWorld(Intermediates,Intermediates.TangentToLocal);
	Intermediates.TangentToWorldSign = TangentSign * GetPrimitive_DeterminantSign_FromFlags(GetPrimitiveData(Intermediates).Flags);	
	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
half3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.TangentToLocal;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	const float3 VertexPosition = GetVertexPosition(Input).Position;
	const float4 WorldPosition = CalcWorldPosition(VertexPosition, GetPrimitiveData(Intermediates).LocalToWorld);
	return WorldPosition;
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return GetVertexPosition(Input).Position;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

	SetTangents(Interpolants, Intermediates.TangentToWorld[0], Intermediates.TangentToWorld[2], Intermediates.TangentToWorldSign);
	SetColor(Interpolants, Intermediates.VertexColors);

	FHairCardsPoint CardsPoint = GetVertexPosition(Input);
	Interpolants.HairPrimitiveLength		= CardsPoint.CardLength;
	Interpolants.HairPrimitiveGroupIndex	= CardsPoint.GroupIndex;
	Interpolants.HairPrimitiveMaterial		= CardsPoint.Materials;

	GetRootUVsAtlasUVs(Input, Interpolants.HairPrimitiveUV, Interpolants.HairPrimitiveRootUV);

	SetSceneData(Interpolants, Intermediates.SceneData);

#if NUM_TEX_COORD_INTERPOLATORS && NUM_MATERIAL_TEXCOORDS_VERTEX
	float2 TexCoords[NUM_MATERIAL_TEXCOORDS_VERTEX];
	GetTexCoords(Input, TexCoords);
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < min(NUM_MATERIAL_TEXCOORDS_VERTEX, NUM_TEX_COORD_INTERPOLATORS); CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, TexCoords[CoordinateIndex]);
	}
#endif

#if NUM_TEX_COORD_INTERPOLATORS 
	SetUV(Interpolants, 0, Interpolants.HairPrimitiveUV);
#endif

	return Interpolants;
}

/** for depth-only pass (Not used by the actual hair shaders)*/
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input)
{
	return 0;
}

// @return translated world position (without quad extension/reorientation).This is used only for velocity computation
float4 VertexFactoryGetWorldPositionRaw(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TransformLocalToTranslatedWorld(GetVertexPosition(Input).Position, GetPrimitiveData(Intermediates).LocalToWorld);
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	const float3 VertexPosition = GetVertexPreviousPosition(Input);
	return DFTransformLocalToTranslatedWorld(VertexPosition, GetPrimitiveData(Intermediates).PreviousLocalToWorld, ResolvedView.PrevPreViewTranslation);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return GetVertexPreviousPosition(Input);
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input)
{
	return float3(0, 0, 1); 
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float3(0,0,1);
}

////////////////////////////////////////////////////////////////////////////////////////////////
// Tesselation support

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

float2 VertexFactoryGetRayTracingTextureCoordinate(FVertexFactoryRayTracingInterpolants Interpolants)
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0, 0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	// Default initialize. Otherwise, some graphics pipelines that
	// couple tessellation with geometry shaders won't write to all TEXCOORD semantics,
	// but read from them when <FVertexFactoryRayTracingInterpolants> is being copied as a whole.
	FVertexFactoryRayTracingInterpolants O = (FVertexFactoryRayTracingInterpolants)0;

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.InstanceId = a.InterpolantsVSToPS.InstanceId;
#if NEEDS_LIGHTMAP_COORDINATE
	O.InterpolantsVSToPS.LightmapDataIndex = a.InterpolantsVSToPS.LightmapDataIndex;
#endif
#endif	
	// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
	// interpolated normal from 'whatever' interpolation scheme we're using

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif
#if	NEEDS_PER_INSTANCE_PARAMS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.PerInstanceParams);
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapCoordinate);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
		for (int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++tc)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
		}
#elif USE_PARTICLE_SUBUVS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[0]);
#endif

#if HAIR_CARD_MESH_FACTORY
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveUV);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveRootUV);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveLength);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveGroupIndex);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveMaterial);
#endif

	return O;
}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
