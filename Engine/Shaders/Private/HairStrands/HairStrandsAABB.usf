// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsAABBCommon.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"

#if SHADER_AABBUPDATE 

uint MacroGroupId;
uint MacroGroupCount;

float PixelSizeAtDepth1;
float NumPixelPerVoxel;
uint  VoxelPageResolution;
uint  PassCount;
float3 TranslatedWorldOffsetCorrection;

// Group data packed as {uint3 AABBMin, uint3 AABBMax}
Buffer<uint> RegisteredIndexBuffer;
Buffer<int> InGroupAABBBuffer;
RWBuffer<int> OutMacroGroupAABBBuffer;
RWBuffer<float> OutMacroGroupVoxelSizeBuffer;

void UpdateValue(inout FHairAABB B, in uint Index, in Buffer<int> InBuffer)
{
	FHairAABB A = ReadHairAABB(Index, InBuffer);
	B = UnionHairAABB(A, B);
}
groupshared float3 g_MinBounds[GROUP_SIZE];
groupshared float3 g_MaxBounds[GROUP_SIZE];
[numthreads(GROUP_SIZE, 1, 1)]
void Main(uint2 DispatchThreadId : SV_DispatchThreadID, uint LinearIndex : SV_GroupIndex)
{
	const uint ThreadId = LinearIndex;
	for (uint PassIt = 0; PassIt < PassCount; ++PassIt)
	{
		const uint PassOffset = PassIt * GROUP_SIZE;
		const uint BoundIndex = PassOffset + ThreadId;
		if (BoundIndex < MacroGroupCount)
		{
			const uint RegisteredIndex = RegisteredIndexBuffer[BoundIndex];
			FHairAABB Bound = ReadHairAABB(RegisteredIndex, InGroupAABBBuffer);
			
			// Correct View0 translated world offset to ViewX translated world offset
			Bound.Min += TranslatedWorldOffsetCorrection;
			Bound.Max += TranslatedWorldOffsetCorrection;

			if (PassIt == 0)
			{
				g_MinBounds[LinearIndex] = Bound.Min;
				g_MaxBounds[LinearIndex] = Bound.Max;
			}
			else
			{
				g_MinBounds[LinearIndex].x = min(g_MinBounds[LinearIndex].x, Bound.Min.x);
				g_MinBounds[LinearIndex].y = min(g_MinBounds[LinearIndex].y, Bound.Min.y);
				g_MinBounds[LinearIndex].z = min(g_MinBounds[LinearIndex].z, Bound.Min.z);

				g_MaxBounds[LinearIndex].x = max(g_MaxBounds[LinearIndex].x, Bound.Max.x);
				g_MaxBounds[LinearIndex].y = max(g_MaxBounds[LinearIndex].y, Bound.Max.y);
				g_MaxBounds[LinearIndex].z = max(g_MaxBounds[LinearIndex].z, Bound.Max.z);
			}
		}
	}
	GroupMemoryBarrierWithGroupSync();
	
	if (ThreadId == 0)
	{
		FHairAABB Bound = InitHairAABB();
		const uint BoundCount = min(GROUP_SIZE, MacroGroupCount);
		for (uint BoundIt = 0; BoundIt < BoundCount; ++BoundIt)
		{
			FHairAABB CurrBound;
			CurrBound.Min = g_MinBounds[BoundIt];
			CurrBound.Max = g_MaxBounds[BoundIt];
			Bound = UnionHairAABB(Bound, CurrBound);
		}

		// Transform positions in camera space
		const FHairAABB ViewBound = Transform(Bound, View.TranslatedWorldToCameraView);

		// Compute voxel size to have 1px = 1 voxels
		const float DefaultNearClip = 0.001f;
		const float PixelWorldSize = PixelSizeAtDepth1 * max(ViewBound.Min.z, DefaultNearClip);
		float VoxelWorldSize = PixelWorldSize * NumPixelPerVoxel;
		WriteHairAABB(MacroGroupId, Bound, OutMacroGroupAABBBuffer);

		// Ensure the MacroGroup AABB has, at minimum, the size of a voxel page
		const float3 Extend = Bound.Max - Bound.Min;
		const float MaxExtent = max3(Extend.x, Extend.y, Extend.z);
		const float BoundVoxelCount = MaxExtent / VoxelWorldSize;
		if (BoundVoxelCount < float(VoxelPageResolution))
		{
			VoxelWorldSize = MaxExtent / float(VoxelPageResolution);
		}

		OutMacroGroupVoxelSizeBuffer[MacroGroupId] = max(HAIR_MIN_VOXEL_SIZE, VoxelWorldSize);
	}
}

#endif

