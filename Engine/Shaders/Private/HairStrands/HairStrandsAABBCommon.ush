// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_AABB_PRECISION_SCALE 1000.f

struct FHairAABB
{
	float3 Min;
	float3 Max;
};

float3 GetCenter(const FHairAABB In)
{
	return (In.Min + In.Max) * 0.5f;
}

float3 GetExtents(const FHairAABB In)
{
	return (In.Max - In.Min) * 0.5f;
}

FHairAABB InitHairAABB()
{
	FHairAABB O;
	O.Min = float3(2147483647, 2147483647, 2147483647);
	O.Max = float3(-2147483647, -2147483647, -2147483647);
	return O;
}


FHairAABB Transform(const FHairAABB In, float4x4 T)
{
	float4 P00 = float4(In.Min.x, In.Min.y, In.Min.z, 1);
	float4 P10 = float4(In.Max.x, In.Min.y, In.Min.z, 1);
	float4 P20 = float4(In.Max.x, In.Max.y, In.Min.z, 1);
	float4 P30 = float4(In.Min.x, In.Max.y, In.Min.z, 1);

	float4 P01 = float4(In.Min.x, In.Min.y, In.Max.z, 1);
	float4 P11 = float4(In.Max.x, In.Min.y, In.Max.z, 1);
	float4 P21 = float4(In.Max.x, In.Max.y, In.Max.z, 1);
	float4 P31 = float4(In.Min.x, In.Max.y, In.Max.z, 1);

	P00 = mul(P00, T); P00 /= P00.w;
	P10 = mul(P10, T); P10 /= P10.w;
	P20 = mul(P20, T); P20 /= P20.w;
	P30 = mul(P30, T); P30 /= P30.w;

	P01 = mul(P01, T); P01 /= P01.w;
	P11 = mul(P11, T); P11 /= P11.w;
	P21 = mul(P21, T); P21 /= P21.w;
	P31 = mul(P31, T); P31 /= P31.w;

	FHairAABB Out;
	Out.Min = min(P00.xyz, min(P10.xyz, min(P20.xyz, P30.xyz)));
	Out.Min = min(P01.xyz, min(P11.xyz, min(P21.xyz, min(P31.xyz, Out.Min))));

	Out.Max = max(P00.xyz, max(P10.xyz, max(P20.xyz, P30.xyz)));
	Out.Max = max(P01.xyz, max(P11.xyz, max(P21.xyz, max(P31.xyz, Out.Max))));
	return Out;
}

FHairAABB ReadHairAABB(uint Index, RWBuffer<int> InBuffer, float PrecisionScale = HAIR_AABB_PRECISION_SCALE)
{
	const uint Index6 = Index * 6;
	FHairAABB O;
	O.Min.x = InBuffer[Index6 + 0] / PrecisionScale;
	O.Min.y = InBuffer[Index6 + 1] / PrecisionScale;
	O.Min.z = InBuffer[Index6 + 2] / PrecisionScale;

	O.Max.x = InBuffer[Index6 + 3] / PrecisionScale;
	O.Max.y = InBuffer[Index6 + 4] / PrecisionScale;
	O.Max.z = InBuffer[Index6 + 5] / PrecisionScale;
	return O;
}

FHairAABB ReadHairAABB(uint Index, Buffer<int> InBuffer, float PrecisionScale = HAIR_AABB_PRECISION_SCALE)
{
	const uint Index6 = Index * 6;
	FHairAABB O;
	O.Min.x = InBuffer[Index6 + 0] / PrecisionScale;
	O.Min.y = InBuffer[Index6 + 1] / PrecisionScale;
	O.Min.z = InBuffer[Index6 + 2] / PrecisionScale;

	O.Max.x = InBuffer[Index6 + 3] / PrecisionScale;
	O.Max.y = InBuffer[Index6 + 4] / PrecisionScale;
	O.Max.z = InBuffer[Index6 + 5] / PrecisionScale;
	return O;
}

void WriteHairAABB(uint Index, FHairAABB In, RWBuffer<int> OutBuffer, float PrecisionScale = HAIR_AABB_PRECISION_SCALE)
{
	const uint Index6 = Index * 6;
	OutBuffer[Index6 + 0] = int(In.Min.x * PrecisionScale);
	OutBuffer[Index6 + 1] = int(In.Min.y * PrecisionScale);
	OutBuffer[Index6 + 2] = int(In.Min.z * PrecisionScale);

	OutBuffer[Index6 + 3] = int(In.Max.x * PrecisionScale);
	OutBuffer[Index6 + 4] = int(In.Max.y * PrecisionScale);
	OutBuffer[Index6 + 5] = int(In.Max.z * PrecisionScale);
}

FHairAABB UnionHairAABB(FHairAABB A, FHairAABB B)
{
	FHairAABB O;
	O.Min.x = min(A.Min.x, B.Min.x);
	O.Min.y = min(A.Min.y, B.Min.y);
	O.Min.z = min(A.Min.z, B.Min.z);

	O.Max.x = max(A.Max.x, B.Max.x);
	O.Max.y = max(A.Max.y, B.Max.y);
	O.Max.z = max(A.Max.z, B.Max.z);
	return O;
}

void InterlockHairAABB(uint Index, FHairAABB In, RWBuffer<int> OutBuffer, float PrecisionScale = HAIR_AABB_PRECISION_SCALE)
{
	const uint Index6 = Index * 6;
	InterlockedMin(OutBuffer[Index6 + 0], int(In.Min.x * PrecisionScale));
	InterlockedMin(OutBuffer[Index6 + 1], int(In.Min.y * PrecisionScale));
	InterlockedMin(OutBuffer[Index6 + 2], int(In.Min.z * PrecisionScale));

	InterlockedMax(OutBuffer[Index6 + 3], int(In.Max.x * PrecisionScale));
	InterlockedMax(OutBuffer[Index6 + 4], int(In.Max.y * PrecisionScale));
	InterlockedMax(OutBuffer[Index6 + 5], int(In.Max.z * PrecisionScale));
}
