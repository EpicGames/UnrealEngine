// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "/Engine/Private/HairStrands/HairStrandsPack.ush"

////////////////////////////////////////////////////////////////////////////////
// Data stride in bytes
// Data stride in bytes
#define HAIR_CURVE_ATTRIBUTE_STRIDE_ROOTUV		4
#define HAIR_CURVE_ATTRIBUTE_STRIDE_SEED		1
#define HAIR_CURVE_ATTRIBUTE_STRIDE_LENGTH		2
#define HAIR_CURVE_ATTRIBUTE_STRIDE_CLUMPID		2
#define HAIR_CURVE_ATTRIBUTE_STRIDE_CLUMPID3	8

#define HAIR_POINT_ATTRIBUTE_STRIDE_COLOR		4
#define HAIR_POINT_ATTRIBUTE_STRIDE_ROUGHNESS	1
#define HAIR_POINT_ATTRIBUTE_STRIDE_AO			1

// Data offset in bytes. The order needs to match HairStrandsDefinitions.h
#define HAIR_CURVE_ATTRIBUTE_OFFSET_ROOTUV(InOffsets)		(InOffsets[0].x)
#define HAIR_CURVE_ATTRIBUTE_OFFSET_SEED(InOffsets)			(InOffsets[0].y)
#define HAIR_CURVE_ATTRIBUTE_OFFSET_LENGTH(InOffsets)		(InOffsets[0].z)
#define HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID(InOffsets)		(InOffsets[0].w)
#define HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID3(InOffsets)		(InOffsets[1].x)

#define HAIR_POINT_ATTRIBUTE_OFFSET_COLOR(InOffsets)		(InOffsets[0].x)
#define HAIR_POINT_ATTRIBUTE_OFFSET_ROUGHNESS(InOffsets)	(InOffsets[0].y)
#define HAIR_POINT_ATTRIBUTE_OFFSET_AO(InOffsets)			(InOffsets[0].z)

////////////////////////////////////////////////////////////////////////////////
// Attribute access functions
uint InternalGetHairStrandsAttributes(uint Index, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, ByteAddressBuffer InAttributeBuffer, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	// Adapt index and offset
	const uint ChunkIndex   = Index >> InIndexToChunkDivAsShift;
	Index                  -= ChunkIndex * InChunkElementCount;
	AttributeOffsetInBytes += ChunkIndex * InChunkStrideInBytes;

	// Ensure the reading address is 4-byte aligned
	const uint Address = AttributeOffsetInBytes + Index * AttributeStrideInBytes;
	const uint Address_4BytesAligned = Address & (~0x3);
	const uint Packed = InAttributeBuffer.Load(Address_4BytesAligned);

	// If the data has a stride < 4bytes, compute the relative offset for reading the data
	const uint ElementIndex = Address - Address_4BytesAligned;
	return Packed >> (8u * ElementIndex);
}

uint2 InternalGetHairStrandsAttributes64(uint Index, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, ByteAddressBuffer InAttributeBuffer, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	const uint ChunkIndex   = Index >> InIndexToChunkDivAsShift;
	Index                  -= ChunkIndex * InChunkElementCount;
	AttributeOffsetInBytes += ChunkIndex * InChunkStrideInBytes;

	// Ensure the reading address is 4-byte aligned
	const uint Address = AttributeOffsetInBytes + Index * AttributeStrideInBytes;
	const uint Address_4BytesAligned = Address & (~0x3);

	// For now, do two separate load to as we don't ensure the offset is 2byte-aligned
	uint2 OutPacked = 0;
	OutPacked.x = InAttributeBuffer.Load(Address_4BytesAligned);
	OutPacked.y = InAttributeBuffer.Load(Address_4BytesAligned + 4u);
	return OutPacked;
}

uint InternalGetHairStrandsAttributesPerCurve(uint HairControlPointId, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, ByteAddressBuffer InAttributeBuffer, ByteAddressBuffer InPointToCurveBuffer, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	const uint CurveIndex = ReadHairPointToCurveIndex(InPointToCurveBuffer, HairControlPointId);
	return InternalGetHairStrandsAttributes(CurveIndex, AttributeOffsetInBytes, AttributeStrideInBytes, InAttributeBuffer, InIndexToChunkDivAsShift, InChunkElementCount, InChunkStrideInBytes);
}

uint2 InternalGetHairStrandsAttributesPerCurve64(uint HairControlPointId, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, ByteAddressBuffer InAttributeBuffer, ByteAddressBuffer InPointToCurveBuffer, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	const uint CurveIndex = ReadHairPointToCurveIndex(InPointToCurveBuffer, HairControlPointId);
	return InternalGetHairStrandsAttributes64(CurveIndex, AttributeOffsetInBytes, AttributeStrideInBytes, InAttributeBuffer, InIndexToChunkDivAsShift, InChunkElementCount, InChunkStrideInBytes);
}

uint InternalGetHairStrandsAttributesPerVertex(uint HairControlPointId, uint AttributeOffsetInBytes, uint AttributeStrideInBytes, ByteAddressBuffer InAttributeBuffer, uint InIndexToChunkDivAsShift, uint InChunkElementCount, uint InChunkStrideInBytes)
{
	return InternalGetHairStrandsAttributes(HairControlPointId, AttributeOffsetInBytes, AttributeStrideInBytes, InAttributeBuffer, InIndexToChunkDivAsShift, InChunkElementCount, InChunkStrideInBytes);
}

float InternalGetHairStrandsCoordU(ByteAddressBuffer InPositionBuffer, uint InPointIndex, float InRadius, float InRootScale, float InTipScale)
{
	return ReadHairControlPoint(InPositionBuffer, InPointIndex, float3(0, 0, 0), InRadius, InRootScale, InTipScale).UCoord;
}

float InternalGetHairStrandsWorldRadius(ByteAddressBuffer InPositionBuffer, uint InPointIndex, float InRadius, float InRootScale, float InTipScale)
{
	return ReadHairControlPoint(InPositionBuffer, InPointIndex, float3(0, 0, 0), InRadius, InRootScale, InTipScale).WorldRadius;
}

bool IsAttributeValid(uint InOffset)
{
	return InOffset != HAIR_ATTRIBUTE_INVALID_OFFSET;
}

////////////////////////////////////////////////////////////////////////////////
// Attribute access with explicit resources

#include "/Engine/Private/HairStrands/HairStrandsVertexFactoryCommon.ush"

#if HAIR_STRAND_MESH_FACTORY
	#if HAIR_STRANDS_PARAMETERS == 1
		// Global constants access
		#define HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name) HairStrandsVF_##Name
	#else
		// Uniform Buffer access
		#define HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name) HairStrandsVF.##Name
	#endif
#include "/Engine/Private/HairStrands/HairStrandsAttributeTemplate.ush"
#endif
