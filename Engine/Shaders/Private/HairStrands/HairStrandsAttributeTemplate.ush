// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

// Check parameters/resources prefixing macro is well defined
#ifndef HAIR_STRANDS_ATTRIBUTE_ACCESSORS
#error HAIR_STRANDS_ATTRIBUTE_ACCESSORS needs to be defined
#endif

// Redefine macro for easing reability
#define PREFIX(Name) HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name)

float2 GetHairStrandsUV(uint InControlPointId, float2 InSegmentUV)
{
	const float CoordU0 = InternalGetHairStrandsCoordU(PREFIX(PositionBuffer), InControlPointId,   PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));
	const float CoordU1 = InternalGetHairStrandsCoordU(PREFIX(PositionBuffer), InControlPointId+1, PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));

	float2 Out;
	Out.x = lerp(CoordU0, CoordU1, InSegmentUV.x);
	Out.y = InSegmentUV.y;
	return Out;
}
float2 GetHairStrandsDimensions(uint InControlPointId, float InSegmentU)
{
	const float CurveLength = UnpackHairLength(InternalGetHairStrandsAttributesPerCurve(InControlPointId, HAIR_CURVE_ATTRIBUTE_OFFSET_LENGTH(PREFIX(CurveAttributeOffsets)), HAIR_CURVE_ATTRIBUTE_STRIDE_LENGTH, PREFIX(CurveAttributeBuffer), PREFIX(PointToCurveBuffer), PREFIX(CurveAttributeIndexToChunkDivAsShift), PREFIX(CurveAttributeChunkElementCount), PREFIX(CurveAttributeChunkStrideInBytes)));

	const float CoordU0 = InternalGetHairStrandsCoordU(PREFIX(PositionBuffer), InControlPointId,   PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));
	const float CoordU1 = InternalGetHairStrandsCoordU(PREFIX(PositionBuffer), InControlPointId+1, PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));
	const float CoordU  = lerp(CoordU0, CoordU1, InSegmentU);

	const float Radius0 = InternalGetHairStrandsWorldRadius(PREFIX(PositionBuffer), InControlPointId,   PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));
	const float Radius1 = InternalGetHairStrandsWorldRadius(PREFIX(PositionBuffer), InControlPointId+1, PREFIX(Radius), PREFIX(RootScale), PREFIX(TipScale));
	const float Radius  = lerp(Radius0, Radius1, InSegmentU);

	return float2(CurveLength * CoordU, Radius);
}
float2 GetHairStrandsRootUV(uint InControlPointId)
{
	const uint RootUVOffset = HAIR_CURVE_ATTRIBUTE_OFFSET_ROOTUV(PREFIX(CurveAttributeOffsets));
	if (IsAttributeValid(RootUVOffset))
	{
		return UnpackHairRootUV(InternalGetHairStrandsAttributesPerCurve(InControlPointId, RootUVOffset, HAIR_CURVE_ATTRIBUTE_STRIDE_ROOTUV, PREFIX(CurveAttributeBuffer), PREFIX(PointToCurveBuffer), PREFIX(CurveAttributeIndexToChunkDivAsShift), PREFIX(CurveAttributeChunkElementCount), PREFIX(CurveAttributeChunkStrideInBytes)));
	}
	else
	{
		return 0;
	}
}
float GetHairStrandsSeed(uint InControlPointId)
{
	return UnpackHairSeed(InternalGetHairStrandsAttributesPerCurve(InControlPointId, HAIR_CURVE_ATTRIBUTE_OFFSET_SEED(PREFIX(CurveAttributeOffsets)), HAIR_CURVE_ATTRIBUTE_STRIDE_SEED, PREFIX(CurveAttributeBuffer), PREFIX(PointToCurveBuffer), PREFIX(CurveAttributeIndexToChunkDivAsShift), PREFIX(CurveAttributeChunkElementCount), PREFIX(CurveAttributeChunkStrideInBytes)));
}
uint3 GetHairStrandsClumpID(uint InControlPointId)
{
	const uint ClumpIdOffset  = HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID(PREFIX(CurveAttributeOffsets));
	const uint ClumpId3Offset = HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID3(PREFIX(CurveAttributeOffsets));
	if (IsAttributeValid(ClumpIdOffset))
	{
		return UnpackHairClumpID(InternalGetHairStrandsAttributesPerCurve(InControlPointId, ClumpIdOffset, HAIR_CURVE_ATTRIBUTE_STRIDE_CLUMPID, PREFIX(CurveAttributeBuffer), PREFIX(PointToCurveBuffer), PREFIX(CurveAttributeIndexToChunkDivAsShift), PREFIX(CurveAttributeChunkElementCount), PREFIX(CurveAttributeChunkStrideInBytes)));
	}
	else if (IsAttributeValid(ClumpId3Offset))
	{
		return UnpackHairClumpID(InternalGetHairStrandsAttributesPerCurve64(InControlPointId, ClumpId3Offset, HAIR_CURVE_ATTRIBUTE_STRIDE_CLUMPID3, PREFIX(CurveAttributeBuffer), PREFIX(PointToCurveBuffer), PREFIX(CurveAttributeIndexToChunkDivAsShift), PREFIX(CurveAttributeChunkElementCount), PREFIX(CurveAttributeChunkStrideInBytes)));
	}
	else
	{
		return 0;
	}
}
float3 GetHairStrandsColor(uint InControlPointId, float InSegmentU)
{
	const uint ColorOffset = HAIR_POINT_ATTRIBUTE_OFFSET_COLOR(PREFIX(PointAttributeOffsets));
	if (IsAttributeValid(ColorOffset))
	{
		const float3 Color0 = UnpackHairColor(InternalGetHairStrandsAttributesPerVertex(InControlPointId,   ColorOffset, HAIR_POINT_ATTRIBUTE_STRIDE_COLOR, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		const float3 Color1 = UnpackHairColor(InternalGetHairStrandsAttributesPerVertex(InControlPointId+1, ColorOffset, HAIR_POINT_ATTRIBUTE_STRIDE_COLOR, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		return lerp(Color0, Color1, InSegmentU);
	}
	else
	{
		return 0;
	}
}
float GetHairStrandsRoughness(uint InControlPointId, float InSegmentU)
{
	const uint RoughnessOffset = HAIR_POINT_ATTRIBUTE_OFFSET_ROUGHNESS(PREFIX(PointAttributeOffsets));
	if (IsAttributeValid(RoughnessOffset))
	{
		const float Roughness0 = UnpackHairRoughness(InternalGetHairStrandsAttributesPerVertex(InControlPointId,   RoughnessOffset, HAIR_POINT_ATTRIBUTE_STRIDE_ROUGHNESS, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		const float Roughness1 = UnpackHairRoughness(InternalGetHairStrandsAttributesPerVertex(InControlPointId+1, RoughnessOffset, HAIR_POINT_ATTRIBUTE_STRIDE_ROUGHNESS, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		return lerp(Roughness0, Roughness1, InSegmentU);
	}
	else
	{
		return 0;
	}
}
float GetHairStrandsAO(uint InControlPointId, float InSegmentU)
{
	const uint AoOffset = HAIR_POINT_ATTRIBUTE_OFFSET_AO(PREFIX(PointAttributeOffsets));
	if (IsAttributeValid(AoOffset))
	{
		const float AO0 = UnpackHairAO(InternalGetHairStrandsAttributesPerVertex(InControlPointId, AoOffset, HAIR_POINT_ATTRIBUTE_STRIDE_AO, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		const float AO1 = UnpackHairAO(InternalGetHairStrandsAttributesPerVertex(InControlPointId+1, AoOffset, HAIR_POINT_ATTRIBUTE_STRIDE_AO, PREFIX(PointAttributeBuffer), PREFIX(PointAttributeIndexToChunkDivAsShift), PREFIX(PointAttributeChunkElementCount), PREFIX(PointAttributeChunkStrideInBytes)));
		return lerp(AO0, AO1, InSegmentU);
	}
	else
	{
		return 1.f;
	}
}
float GetHairStrandsDepth() 
{ 
	return 0.f; 
} 
float GetHairStrandsCoverage()
{
	return 1;
}
float GetHairStrandsGroupIndex() 
{ 
	return PREFIX(GroupIndex); 
}
bool HasHairStrandsRootUV() 	{ return IsAttributeValid(HAIR_CURVE_ATTRIBUTE_OFFSET_ROOTUV(PREFIX(CurveAttributeOffsets))); }
bool HasHairStrandsClumpId() 	{ return IsAttributeValid(HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID(PREFIX(CurveAttributeOffsets))); }
bool HasHairStrandsClumpId3() 	{ return IsAttributeValid(HAIR_CURVE_ATTRIBUTE_OFFSET_CLUMPID3(PREFIX(CurveAttributeOffsets))); }
bool HasHairStrandsColor() 		{ return IsAttributeValid(HAIR_POINT_ATTRIBUTE_OFFSET_COLOR(PREFIX(PointAttributeOffsets))); }
bool HasHairStrandsRoughness() 	{ return IsAttributeValid(HAIR_POINT_ATTRIBUTE_OFFSET_ROUGHNESS(PREFIX(PointAttributeOffsets))); }
bool HasHairStrandsAO() 		{ return IsAttributeValid(HAIR_POINT_ATTRIBUTE_OFFSET_AO(PREFIX(PointAttributeOffsets))); }
