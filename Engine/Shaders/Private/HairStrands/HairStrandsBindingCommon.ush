// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HairStrandsPack.ush"

#if defined(NEED_HAIR_MESH_BASIS_PADDING) && NEED_HAIR_MESH_BASIS_PADDING
#define HAIR_MESH_BASIS_PAD(x) x
#else
#define HAIR_MESH_BASIS_PAD(x)
#endif

struct FHairMeshBasis
{
	float3 BaseO;
	HAIR_MESH_BASIS_PAD(float _Pad0;)
	float3 BaseT;
	HAIR_MESH_BASIS_PAD(float _Pad1;)
	float3 BaseB;
	HAIR_MESH_BASIS_PAD(float _Pad2;)
	float3 BaseN;
	HAIR_MESH_BASIS_PAD(float _Pad3;) 
};

struct FHairMeshTriangle
{
	float3 P0;
	float3 P1;
	float3 P2;

	float3 N0;
	float3 N1;
	float3 N2;
};

float3 GetVertexPosition(uint VId, Buffer<float> InPositionBuffer, Buffer<uint> InIndexBuffer, uint InIndexOffset, uint InMaxIndexCount, uint InMaxVertexCount)
{
	const uint Index = min(InIndexOffset + VId, InMaxIndexCount-1);
	uint VertexIndex = InIndexBuffer.Load(Index);
	VertexIndex = min(VertexIndex, InMaxVertexCount-1);

	float3 P = 0;
	P.x = InPositionBuffer.Load(VertexIndex*3 + 0);
	P.y = InPositionBuffer.Load(VertexIndex*3 + 1);
	P.z = InPositionBuffer.Load(VertexIndex*3 + 2);
	return P;
}

float3 GetVertexNormal(uint VId, Buffer<float4> InTangentBuffer, Buffer<uint> InIndexBuffer, uint InIndexOffset, uint InMaxIndexCount, uint InMaxVertexCount, uint InTangentFormat)
{
	const uint Index = min(InIndexOffset + VId, InMaxIndexCount-1);
	uint VertexIndex = InIndexBuffer.Load(Index);
	VertexIndex = min(VertexIndex, InMaxVertexCount-1);

	float4 TangentZ = 0;
	if (InTangentFormat == 1)
	{
		TangentZ = float4(InTangentBuffer[VertexIndex].xyz,0);
	}
	else
	{
		TangentZ = TangentBias(InTangentBuffer[VertexIndex * 2 + 1].xyzw);
	}
	return TangentZ.xyz;
}

// Return a mesh triangle (position + normal)
FHairMeshTriangle GetHairMeshTriangle(uint TriangleIndex, Buffer<float> InPositionBuffer, Buffer<float4> InTangentBuffer, Buffer<uint> InIndexBuffer, uint InIndexOffset, uint InMaxIndexCount, uint InMaxVertexCount, uint InTangentFormat, bool bUseVertexTangent)
{
	const uint BaseVertexId = TriangleIndex * 3;
	const uint VId0 = BaseVertexId;
	const uint VId1 = BaseVertexId + 1;
	const uint VId2 = BaseVertexId + 2;

	FHairMeshTriangle Out = (FHairMeshTriangle)0;
	Out.P0 = GetVertexPosition(VId0, InPositionBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount);
	Out.P1 = GetVertexPosition(VId1, InPositionBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount);
	Out.P2 = GetVertexPosition(VId2, InPositionBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount);

	if (bUseVertexTangent)
	{
		Out.N0 = GetVertexNormal(VId0, InTangentBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount, InTangentFormat);
		Out.N1 = GetVertexNormal(VId1, InTangentBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount, InTangentFormat);
		Out.N2 = GetVertexNormal(VId2, InTangentBuffer, InIndexBuffer, InIndexOffset, InMaxIndexCount, InMaxVertexCount, InTangentFormat);
	}
	
	return Out;
}

// Return a mesh triangle (position + normal)
FHairMeshTriangle GetHairMeshTriangle(uint TriangleIndex, Buffer<float4> InPositionBuffer)
{
	FHairMeshTriangle Out;
	const float4 V0 = InPositionBuffer[TriangleIndex * 3 + 0];
	const float4 V1 = InPositionBuffer[TriangleIndex * 3 + 1];
	const float4 V2 = InPositionBuffer[TriangleIndex * 3 + 2];
	Out.P0 = V0.xyz;
	Out.P1 = V1.xyz;
	Out.P2 = V2.xyz;
	Out.N0 = UnpackVertexNormal(V0.w);
	Out.N1 = UnpackVertexNormal(V1.w);
	Out.N2 = UnpackVertexNormal(V2.w);
	return Out;
}

// Return a hair basis (origin + tangent frame)
FHairMeshBasis GetHairMeshBasis(
	uint TriangleIndex,
	Buffer<float4> InPositionBuffer,
	float3 B/*Barycentrics*/)
{
	FHairMeshBasis Out;

	const float4 V0 = InPositionBuffer[TriangleIndex * 3 + 0];
	const float4 V1 = InPositionBuffer[TriangleIndex * 3 + 1];
	const float4 V2 = InPositionBuffer[TriangleIndex * 3 + 2];

	const float3 N0 = UnpackVertexNormal(V0.w);
	const float3 N1 = UnpackVertexNormal(V1.w);
	const float3 N2 = UnpackVertexNormal(V2.w);

	const float3 E0 = (V1 - V0).xyz;
	const float3 E1 = (V2 - V0).xyz;

	// Build orthonormal frame
	Out.BaseO = (V0 * B.x + V1 * B.y + V2 * B.z).xyz;
	Out.BaseN = normalize(N0 * B.x + N1 * B.y + N2 * B.z);
	Out.BaseT = normalize(cross(Out.BaseN, E1));
	Out.BaseB = normalize(cross(Out.BaseT, Out.BaseN));

	const bool bUseFaceNormal = false;
	if (bUseFaceNormal)
	{
		Out.BaseN = normalize(cross(E1, E0));
		Out.BaseT = normalize(E0);
		Out.BaseB = normalize(cross(Out.BaseT, Out.BaseN));
	}
	return Out;
}

float3 ToTriangle(const float3 P, const FHairMeshBasis T)
{
	const float3 LocalP = P - T.BaseO;

	float3 Out;
	Out.x = dot(LocalP, T.BaseT);
	Out.y = dot(LocalP, T.BaseB);
	Out.z = dot(LocalP, T.BaseN);
	return Out;
}

float3 ToWorld(const float3 P, const FHairMeshBasis T)
{
	const float3 Rotation    = P.x * T.BaseT + P.y * T.BaseB + P.z * T.BaseN;
	const float3 Translation = T.BaseO;
	return Rotation + Translation;

}

float3 VectorToWorld(const float3 P, const FHairMeshBasis T)
{
	const float3 Rotation = P.x * T.BaseT + P.y * T.BaseB + P.z * T.BaseN;
	return Rotation;
}

float3 VectorToTriangle(const float3 P, const FHairMeshBasis T)
{
	float3 Out;
	Out.x = dot(P, T.BaseT);
	Out.y = dot(P, T.BaseB);
	Out.z = dot(P, T.BaseN);
	return Out;
}

float3 TransformPoint(float3 P, const FHairMeshBasis RestBasis, const FHairMeshBasis DeformedBasis)
{
	float3 LocalP = ToTriangle(P, RestBasis);
	return ToWorld(LocalP, DeformedBasis);
}

// Apply RBF weight onto a control point
float3 ApplyRBF(
	float3 RestControlPoint,
	int InMaxSampleCount,
	StructuredBuffer<float4> InRestSamplePositionsBuffer,
	StructuredBuffer<float4> InMeshSampleWeightsBuffer,
	StructuredBuffer<float4> InDeformedSamplePositionsBuffer,
	uint RBFLocalSpace)
{
	const bool bEnableLocalSpace = (InMaxSampleCount > 0) && (RBFLocalSpace == 1);
	const float4 RestSampleOffset = bEnableLocalSpace ? InRestSamplePositionsBuffer[0] : float4(0,0,0,0);
	const float4 DeformedSampleOffset = bEnableLocalSpace ? InDeformedSamplePositionsBuffer[0] : float4(0,0,0,0);
	
	float3 ControlPoint = RestControlPoint-RestSampleOffset.xyz;
	// Apply rbf interpolation from the samples set
	for (int i = 0; i < InMaxSampleCount; ++i)
	{
		const float3 PositionDelta = RestControlPoint - InRestSamplePositionsBuffer[i].xyz;
		const float FunctionValue = sqrt(dot(PositionDelta, PositionDelta) + 1);
		ControlPoint += FunctionValue * InMeshSampleWeightsBuffer[i].xyz;
	}
	ControlPoint += InMeshSampleWeightsBuffer[InMaxSampleCount].xyz;
	ControlPoint += InMeshSampleWeightsBuffer[InMaxSampleCount + 1].xyz * (RestControlPoint.x-RestSampleOffset.x);
	ControlPoint += InMeshSampleWeightsBuffer[InMaxSampleCount + 2].xyz * (RestControlPoint.y-RestSampleOffset.y);
	ControlPoint += InMeshSampleWeightsBuffer[InMaxSampleCount + 3].xyz * (RestControlPoint.z-RestSampleOffset.z);

	ControlPoint += DeformedSampleOffset.xyz;
	return ControlPoint;
}
