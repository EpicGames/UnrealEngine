// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"

////////////////////////////////////////////////////////////////////////////////
// Defines

// Define the Group count along X, when the 1D number of groups exceed the 65k limits. 
// In such a case, the 1D dispatch is wrapped into a 2D dispatch, whose the X dimention 
// has a size of INDIRECT_GROUP_COUNT_X, with a total of INDIRECT_GROUP_COUNT_X * GROUP_SIZE 
// thread per line
#define INDIRECT_GROUP_COUNT_X 16

// Max number of discrete LOD that a hair group can have
#define MAX_HAIR_LOD 8

////////////////////////////////////////////////////////////////////////////////
// Utils

// When culling is disabled: return the vertex index at which the control point should be fetched.
uint GetHairStrandsVertexFetchIndex(uint2 InDispatchThreadId, uint InGroupSize, uint InGroupCountX)
{
	return InDispatchThreadId.x + InDispatchThreadId.y * InGroupSize * InGroupCountX;
}

// When culling is enabled: return the vertex index at which the control point should be fetched.
uint GetHairStrandsVertexFetchIndex_Culled(uint2 InDispatchThreadId, uint InGroupSize)
{
	return InDispatchThreadId.x + InDispatchThreadId.y * InGroupSize * INDIRECT_GROUP_COUNT_X;
}

////////////////////////////////////////////////////////////////////////////////
// Store info about all LODS of a cluster (total LOD count, screen size, ...)
struct FHairClusterInfo
{
	uint  LODCount;
	uint  LOD_bIsVisible;

	// ScreenSize
	float LOD0_ScreenSize;
	float LOD1_ScreenSize;
	float LOD2_ScreenSize;
	float LOD3_ScreenSize;
	float LOD4_ScreenSize;
	float LOD5_ScreenSize;
	float LOD6_ScreenSize;
	float LOD7_ScreenSize;

	// Scale
	float LOD0_RadiusScale;
	float LOD1_RadiusScale;
	float LOD2_RadiusScale;
	float LOD3_RadiusScale;
	float LOD4_RadiusScale;
	float LOD5_RadiusScale;
	float LOD6_RadiusScale;
	float LOD7_RadiusScale;
};

struct FPackedHairClusterInfo
{
	uint ScreenSizeX123;
	uint ScreenSize4567;
	uint RadiusScaleX123;
	uint RadiusScale4567;
};

FHairClusterInfo UnpackHairClusterInfo(FPackedHairClusterInfo In, float4 InParameters)
{
	const float4 Screen0 = UnpackRGBA8(In.ScreenSizeX123);
	const float4 Screen1 = UnpackRGBA8(In.ScreenSize4567);

	const float4 Radius0 = UnpackRGBA8(In.RadiusScaleX123);
	const float4 Radius1 = UnpackRGBA8(In.RadiusScale4567);


	FHairClusterInfo Out;
	Out.LOD0_ScreenSize = 1.f;
	Out.LOD1_ScreenSize = Screen0.y * InParameters.x + InParameters.y;
	Out.LOD2_ScreenSize = Screen0.z * InParameters.x + InParameters.y;
	Out.LOD3_ScreenSize = Screen0.w * InParameters.x + InParameters.y;
	Out.LOD4_ScreenSize = Screen1.x * InParameters.x + InParameters.y;
	Out.LOD5_ScreenSize = Screen1.y * InParameters.x + InParameters.y;
	Out.LOD6_ScreenSize = Screen1.z * InParameters.x + InParameters.y;
	Out.LOD7_ScreenSize = Screen1.w * InParameters.x + InParameters.y;

	Out.LOD0_RadiusScale = 1.f;
	Out.LOD1_RadiusScale = Radius0.y * InParameters.z + InParameters.w;
	Out.LOD2_RadiusScale = Radius0.z * InParameters.z + InParameters.w;
	Out.LOD3_RadiusScale = Radius0.w * InParameters.z + InParameters.w;
	Out.LOD4_RadiusScale = Radius1.x * InParameters.z + InParameters.w;
	Out.LOD5_RadiusScale = Radius1.y * InParameters.z + InParameters.w;
	Out.LOD6_RadiusScale = Radius1.z * InParameters.z + InParameters.w;
	Out.LOD7_RadiusScale = Radius1.w * InParameters.z + InParameters.w;

	Out.LODCount		= In.ScreenSizeX123 & 0xFFu;
	Out.LOD_bIsVisible  = In.RadiusScaleX123 & 0xFFu;

	return Out;
}

////////////////////////////////////////////////////////////////////////////////

struct FHairClusterLOD
{
	float RadiusScale;
	float LOD;
	uint  LODCount;
};

////////////////////////////////////////////////////////////////////////////////

struct FHairClusterDebugInfo
{
	uint GroupIndex;
	float LOD;
	float VertexCount;
	float CurveCount;
};

////////////////////////////////////////////////////////////////////////////////

// Compute if a hair cluster is visible for a given LOD
bool IsLODVisible(FHairClusterInfo InInfo, float LOD)
{
	const uint iLOD = clamp(floor(LOD), 0, InInfo.LODCount - 1);
	return InInfo.LOD_bIsVisible & (1u << iLOD);
}

// Compute the hair LOD based on the cluster screen size
float GetLOD(FHairClusterInfo InInfo, float InScreenSize, float InLODBias)
{
	float OutLOD = 0;

	if (InScreenSize < InInfo.LOD0_ScreenSize && InInfo.LODCount > 1)
	{
		float ScreenSize[MAX_HAIR_LOD] = 
		{
			InInfo.LOD0_ScreenSize,
			InInfo.LOD1_ScreenSize,
			InInfo.LOD2_ScreenSize,
			InInfo.LOD3_ScreenSize,
			InInfo.LOD4_ScreenSize,
			InInfo.LOD5_ScreenSize,
			InInfo.LOD6_ScreenSize,
			InInfo.LOD7_ScreenSize
		};

		for (uint LODIt = 1; LODIt < InInfo.LODCount; ++LODIt)
		{
			if (InScreenSize >= ScreenSize[LODIt])
			{
				uint PrevLODIt = LODIt - 1;

				const float S_Delta = abs(ScreenSize[PrevLODIt] - ScreenSize[LODIt]);
				const float S = S_Delta > 0 ? saturate(abs(InScreenSize - ScreenSize[LODIt]) / S_Delta) : 0;
				OutLOD = PrevLODIt + (1-S);
				break;
			}
			else if (LODIt == InInfo.LODCount - 1)
			{
				OutLOD = LODIt;
			}
		}
	}

	if (InLODBias != 0)
	{
		OutLOD = clamp(OutLOD + InLODBias, 0, InInfo.LODCount - 1);
	}
	return OutLOD;
}

// Compute the hair cluster LOD for a given LOD level
FHairClusterLOD GetHairClusterLOD(
	FPackedHairClusterInfo InPacked, 
	float4 InClusterInfoParameters,
	float LOD)
{
	const FHairClusterInfo InInfo = UnpackHairClusterInfo(InPacked, InClusterInfoParameters);

	float RadiusScale[MAX_HAIR_LOD] = 
	{
		InInfo.LOD0_RadiusScale,
		InInfo.LOD1_RadiusScale,
		InInfo.LOD2_RadiusScale,
		InInfo.LOD3_RadiusScale,
		InInfo.LOD4_RadiusScale,
		InInfo.LOD5_RadiusScale,
		InInfo.LOD6_RadiusScale,
		InInfo.LOD7_RadiusScale
	};

	const uint iLOD = clamp(floor(LOD), 0, InInfo.LODCount-1);
	const float S = LOD - iLOD;

	FHairClusterLOD Out;
	Out.RadiusScale = lerp(RadiusScale[iLOD], RadiusScale[uint(min(iLOD + 1, InInfo.LODCount - 1))], S);
	Out.LOD = LOD;
	Out.LODCount = InInfo.LODCount;
	return Out;
}

// Compute the screen size of a bounding sphere
// This is the equivalent of ComputeBoundsScreenSize in SceneManagement.h
float ComputeBoundsScreenSize(float3 InSphereOrigin, float InSphereRadius, float3 InViewOrigin, float4x4 InProjMatrix)
{
	const float Dist = distance(InSphereOrigin, InViewOrigin);

	// Get projection multiple accounting for view scaling.
	const float ScreenMultiple = max(0.5f * InProjMatrix[0].x, 0.5f * InProjMatrix[1].y);

	// Calculate screen-space projected radius
	const float ScreenRadius = ScreenMultiple * InSphereRadius / max(1.0f, Dist);

	// For clarity, we end up comparing the diameter
	return ScreenRadius * 2.0f;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Point LOD

uint UnpackPointLOD(uint InPacked, uint InIndex)
{
	return (InPacked >> (InIndex * HAIR_POINT_LOD_BIT_COUNT)) & 0xF;
}

uint GetHairControlPointMinLOD(uint InPointIndex, Buffer<uint> InPointLODBuffer)
{
	const uint BlockIt      = InPointIndex >> HAIR_POINT_LOD_COUNT_PER_UINT_DIV_AS_SHIFT;
	const uint LocalPointIt = InPointIndex - BlockIt * HAIR_POINT_LOD_COUNT_PER_UINT;
	const uint Packed 		= InPointLODBuffer[BlockIt];
	const uint MinLOD 		= UnpackPointLOD(Packed, LocalPointIt);
	return MinLOD;
}

bool IsHairControlPointActive(uint InMinLOD, float InLODIndex)
{
	return uint(floor(InLODIndex)) <= InMinLOD;
}

bool IsHairControlPointActive(uint InPointIndex, Buffer<uint> InPointLODBuffer, float InLODIndex)
{
	const uint MinLOD = GetHairControlPointMinLOD(InPointIndex, InPointLODBuffer);
	return IsHairControlPointActive(MinLOD, InLODIndex);
}