// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "../ColorMap.ush"
#include "HairStrandsAABBCommon.ush"
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

#ifndef USE_HAIR_TRIANGLE_STRIP
	#error Hair triangle geometry type needs to be defined
#endif

#define COMPACTCS_NUMTHREAD 64

// According to https://docs.microsoft.com/en-us/windows/win32/direct3d11/overviews-direct3d-11-devices-downlevel-compute-shaders#thread-group-shared-memory-tgsm
// A thread can access to 32 byte per thread when number of thread in a group is 512. This is largeley enough for this use case. 
#define PREFIXSUM_PARALLEL  512
#define PREFIXSUM_LASTTHREAD (PREFIXSUM_PARALLEL-1)

#define D_USE_CONTINUOUSLOD  1
#define D_LOD_DEBUG_SCALE 1000.0f

// Cluster data in ClusterAABB buffers is packed as {uint3 AABBMin, uint3 AABBMax}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Utility functions

uint3 ComputeDispatchGroupCount2D(uint GroupCount)
{
	const uint DispatchCountX = int(floor(sqrt(float(GroupCount))));
	const uint DispatchCountY = DispatchCountX + DivideAndRoundUp(GroupCount - DispatchCountX * DispatchCountX, DispatchCountX);

	return uint3(DispatchCountX, DispatchCountY, 1);
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// Clear group/cluster AABBs
#ifdef SHADER_CLEARAABB
uint AABBCount;
RWBuffer<int> OutAABBBuffer;

[numthreads(64, 1, 1)]
void MainClearAABBCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x < AABBCount*6)
	{
		const int ClusterId = DispatchThreadId.x / 6;
		const bool bIsMin = (DispatchThreadId.x - ClusterId * 6) < 3 ? 1 : 0;
		checkBufferAccessSlow(OutAABBBuffer, DispatchThreadId.x);
		OutAABBBuffer[DispatchThreadId.x] = bIsMin ? 2147483647 : -2147483648;
	}
}

#endif

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Write out culled index buffex

#if SHADER_CLUSTER_CULL

#define NUM_THREADS_PER_GROUP GROUP_SIZE
#include "../ThreadGroupPrefixSum.ush"

//uint ForceLOD;
uint ClusterGroupIndex;
float LODIndex;
float LODBias;
uint CurveCount;
uint CurvePerGroup;
uint DispatchCountX;
float4 ClusterInfoParameters;

ByteAddressBuffer CurveBuffer;
Buffer<uint> CurveToClusterIdBuffer;
Buffer<uint> PointLODBuffer;
StructuredBuffer<FPackedHairClusterInfo> ClusterInfoBuffer;

RWStructuredBuffer<uint>  OutPointCounter;
RWBuffer<uint>  OutIndexBuffer;
RWByteAddressBuffer  OutCulledCurveBuffer;

#ifndef GROUP_SIZE
#error GROUP_SIZE needs to be Undefined
#endif


#define CURVE_PER_GROUP (GROUP_SIZE / PERMUTATION_POINT_PER_CURVE)
#define MAX_CURVE CURVE_PER_GROUP
#define MAX_POINT PERMUTATION_POINT_PER_CURVE

uint GetGlobalCurveIndex(uint ThreadIndex, uint2 GroupIndex)
{
	const uint LinearGroupIndex = GroupIndex.x + GroupIndex.y * DispatchCountX;
#if PERMUTATION_POINT_PER_CURVE == 64
	return LinearGroupIndex;
#else
	return ThreadIndex + LinearGroupIndex * CURVE_PER_GROUP;
#endif
}

//-------------------------------------------------------------------------------------------------
#if PERMUTATION_POINT_PER_CURVE < 64

// Permutation for curves having less that 32 control points
// In this permutation we process several curves with the same thread group

groupshared uint CurvePointIndex[MAX_CURVE];
groupshared uint CurvePointCount[MAX_CURVE];
groupshared uint ClusterLOD[MAX_CURVE];
groupshared uint LODCount[MAX_CURVE];

groupshared uint AllocationOffset;
groupshared uint AllocationCount;

// Store locally the culled point index (encoded into 8 bits, and store 4 per uint)
groupshared uint CulledPointIndices[MAX_CURVE * MAX_POINT];
groupshared uint ValidPoints[MAX_CURVE][MAX_POINT];

groupshared uint WriteOffset0[MAX_CURVE];
groupshared uint WriteOffsetN[MAX_CURVE];

[numthreads(GROUP_SIZE, 1, 1)]
void Main(uint3 DispatchThreadId : SV_DispatchThreadID, uint ThreadIndex : SV_GroupIndex, uint2 GroupIndex : SV_GroupID)
{
	// 1. Read the Curve data
	if (ThreadIndex < CURVE_PER_GROUP)
	{
		const uint LocalCurveIndex  = ThreadIndex;
		const uint GlobalCurveIndex = GetGlobalCurveIndex(ThreadIndex, GroupIndex);
		const bool bValidCurveIndex = GlobalCurveIndex < CurveCount;
	
		// 1.1 Cluster ID
		// 1.3 Get cluster info (i.e. map all info per LOD for a given cluster ID) and derive effective LOD data
		FHairClusterLOD ClusterLODInfo = (FHairClusterLOD)0;
		if (bValidCurveIndex)
		{
			const uint ClusterId = CurveToClusterIdBuffer[GlobalCurveIndex];
			ClusterLODInfo = GetHairClusterLOD(ClusterInfoBuffer[ClusterId], ClusterInfoParameters, LODIndex);
		}
		ClusterLOD[LocalCurveIndex] 		 = ClusterLODInfo.LOD;
		LODCount[LocalCurveIndex]			 = ClusterLODInfo.LODCount;

		// 1.4 Curve's point count/offset
		CurvePointIndex[LocalCurveIndex] = 0;
		CurvePointCount[LocalCurveIndex] = 0;
		if (bValidCurveIndex)
		{
			const FHairCurve Curve = ReadHairCurve(CurveBuffer, GlobalCurveIndex);
			CurvePointIndex[LocalCurveIndex] = Curve.PointIndex;
			CurvePointCount[LocalCurveIndex] = Curve.PointCount;
		}


	}
	if (ThreadIndex == 0)
	{
		AllocationCount = 0;
		AllocationOffset = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	// 2. Mark valid point and compute their final position
	{
		const uint CurveIndex        = ThreadIndex / uint(MAX_POINT);
		const uint CurrentPointIndex = ThreadIndex % uint(MAX_POINT);

		// 2.2 Mark point having their LOD visibility smaller than the requested LOD level
		ValidPoints[CurveIndex][CurrentPointIndex] = 0; 
		if (CurveIndex < CurveCount) // incorrect
		{	
			if (CurrentPointIndex < CurvePointCount[CurveIndex])
			{
				const uint PointIndex = CurrentPointIndex + CurvePointIndex[CurveIndex];
				const uint MinLOD = GetHairControlPointMinLOD(PointIndex, PointLODBuffer);

				// Prune control point based on their MinLOD value
				if (IsHairControlPointActive(MinLOD, ClusterLOD[CurveIndex]))
				{
					ValidPoints[CurveIndex][CurrentPointIndex] = 1;
				}
			}
		}
	
		// 2.3 Count/Compact visible points
		{
			uint LocalAllocationCount = 0;
			const uint bIsValid = ValidPoints[CurveIndex][CurrentPointIndex];
			const uint PrefixSum = ThreadGroupPrefixSum(bIsValid, ThreadIndex, LocalAllocationCount);
			if (ThreadIndex == 0)
			{
				AllocationCount = LocalAllocationCount;
			}
			if (bIsValid)
			{
				CulledPointIndices[PrefixSum]		= CurrentPointIndex + CurvePointIndex[CurveIndex];
			}

			if (CurveIndex < CurveCount)
			{
				if (CurrentPointIndex == 0) 							WriteOffset0[CurveIndex] = PrefixSum;
				if (CurrentPointIndex == CurvePointCount[CurveIndex]-1)	WriteOffsetN[CurveIndex] = PrefixSum;
			}
		}
	}

	// 4. Global allocate
	if (ThreadIndex == 0)
	{
		InterlockedAdd(OutPointCounter[ClusterGroupIndex], AllocationCount, AllocationOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// 5. Write out index/scale
	if (ThreadIndex < AllocationCount)
	{
		OutIndexBuffer[AllocationOffset + ThreadIndex]       = CulledPointIndices[ThreadIndex];
	}

	// 6. Write out curve data with visible points for subsequent passes
	if (ThreadIndex < CURVE_PER_GROUP)
	{
		const uint LocalCurveIndex  = ThreadIndex;
		const uint GlobalCurveIndex = GetGlobalCurveIndex(ThreadIndex, GroupIndex);
		if (GlobalCurveIndex < CurveCount)
		{
			FHairCurve CulledCurve;
			CulledCurve.PointIndex = WriteOffset0[LocalCurveIndex];
			CulledCurve.PointCount =(WriteOffsetN[LocalCurveIndex] - WriteOffset0[LocalCurveIndex]) + 1;

			WriteHairCurve(OutCulledCurveBuffer, GlobalCurveIndex, CulledCurve);
		}
	}
}
#endif // PERMUTATION_POINT_PER_CURVE < 64
//-------------------------------------------------------------------------------------------------
#if PERMUTATION_POINT_PER_CURVE == 64

// Permutation is for curve having more than 32 control point per curve
// In this permutation we process 1 curve per thread group

groupshared uint CurvePointIndex;
groupshared uint CurvePointCount;
groupshared float ClusterLOD;
groupshared uint LODCount;

groupshared uint AllocationOffset;
groupshared uint AllocationCount;

// Store locally the culled point index (encoded into 8 bits, and store 4 per uint)
groupshared uint CulledPointIndices[HAIR_MAX_NUM_POINT_PER_CURVE]; 

// Version for curve with >32 points
[numthreads(GROUP_SIZE, 1, 1)]
void Main(uint3 DispatchThreadId : SV_DispatchThreadID, uint ThreadIndex : SV_GroupIndex, uint2 GroupIndex : SV_GroupID)
{	
	// 1. Read the Curve data
	const uint CurveIndex = GetGlobalCurveIndex(ThreadIndex, GroupIndex);
	const bool bIsCurveValid = CurveIndex < CurveCount;
	if (!bIsCurveValid)
	{
		return;
	}
	
	if (ThreadIndex == 0)
	{
		// 1.1 Cluster ID
		// 1.2 Get cluster info (i.e. map all info per LOD for a given cluster ID) and derive effective LOD data
		const uint ClusterId = CurveToClusterIdBuffer[CurveIndex];
		const FHairClusterLOD ClusterLODInfo = GetHairClusterLOD(ClusterInfoBuffer[ClusterId], ClusterInfoParameters, LODIndex);
		ClusterLOD = ClusterLODInfo.LOD;
		LODCount = ClusterLODInfo.LODCount;

		// 1.4 Curve's point count/offset
		const FHairCurve Curve = ReadHairCurve(CurveBuffer, CurveIndex);
		CurvePointIndex = Curve.PointIndex;
		CurvePointCount = Curve.PointCount;

		AllocationCount = 0;
		AllocationOffset = 0;
	}
	GroupMemoryBarrierWithGroupSync();


	// 2. Mark valid point and compute their final position
	{
		const uint LoopCount = DivideAndRoundUp(CurvePointCount, GROUP_SIZE);
		for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
		{		
			const uint CurrentPointIndex = ThreadIndex + LoopIt * GROUP_SIZE;
	
			// 2.2 Mark point having their LOD visibility smaller than the requested LOD level
			bool bIsValid = false;
			if (CurrentPointIndex < CurvePointCount)
			{
				const uint PointIndex = CurrentPointIndex + CurvePointIndex;
				const uint MinLOD = GetHairControlPointMinLOD(PointIndex, PointLODBuffer);

				// Prune control point based on their MinLOD value
				if (IsHairControlPointActive(MinLOD, ClusterLOD))
				{
					bIsValid = true;
				}
			}
		
			// 2.3 Count/Compact visible points
			{
				uint IndexOffset = AllocationCount;
				uint LocalAllocationCount = 0;
				const uint PrefixSum = ThreadGroupPrefixSum(uint(bIsValid), ThreadIndex, LocalAllocationCount); // Has a ThreadMemoryGroupSync internally
				if (bIsValid)
				{
					CulledPointIndices[IndexOffset + PrefixSum] = CurrentPointIndex;
				}

				if (ThreadIndex == 0)
				{
					AllocationCount += LocalAllocationCount;
				}
			}

			GroupMemoryBarrierWithGroupSync();
		}
	}

	// 4. Global allocate
	if (ThreadIndex == 0)
	{
		InterlockedAdd(OutPointCounter[ClusterGroupIndex], AllocationCount, AllocationOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// 5. Write out index/scale
	{
		const uint LoopCount = DivideAndRoundUp(AllocationCount, GROUP_SIZE);
		for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
		{
			const uint CurrentPointIndex = ThreadIndex + LoopIt * GROUP_SIZE;
			if (CurrentPointIndex < AllocationCount)
			{
				const uint CullPointIndex = CulledPointIndices[CurrentPointIndex];
				OutIndexBuffer[AllocationOffset + CurrentPointIndex]       = CurvePointIndex + CullPointIndex;
			}
		}
	}

	// 6. Write out curve data with visible points for subsequent passes
	if (ThreadIndex == 0)
	{	
		FHairCurve CulledCurve;
		CulledCurve.PointCount = AllocationCount;
		CulledCurve.PointIndex = AllocationOffset;
		WriteHairCurve(OutCulledCurveBuffer, CurveIndex, CulledCurve);
	}
}
#endif // PERMUTATION_POINT_PER_CURVE == 64
//-------------------------------------------------------------------------------------------------

#endif // SHADER_CLUSTER_CULL

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Build indirect Draw/Dispatch based on culled curve output

#if SHADER_CLUSTER_CULL_ARGS

uint InstanceRegisteredIndex;
uint ClusterGroupIndex;
StructuredBuffer<uint> PointCounterBuffer;
RWBuffer<uint> RWIndirectDrawArgsBuffer;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;
RWBuffer<uint> RWIndirectDispatchArgsGlobalBuffer;

[numthreads(1, 1, 1)]
void Main(uint DispatchThreadId : SV_DispatchThreadID, uint ThreadIndex : SV_GroupIndex)
{
	const uint PointCount = PointCounterBuffer[ClusterGroupIndex];
	const uint DispatchCount = DivideAndRoundUp(PointCount, HAIR_VERTEXCOUNT_GROUP_SIZE);

	RWIndirectDrawArgsBuffer[0] = PointCount * HAIR_POINT_TO_VERTEX;
	RWIndirectDrawArgsBuffer[1] = 1; // 1 instance
	RWIndirectDrawArgsBuffer[2] = 0;
	RWIndirectDrawArgsBuffer[3] = 0;

	RWIndirectDispatchArgsBuffer[0] = DispatchCount;
	RWIndirectDispatchArgsBuffer[1] = 1;
	RWIndirectDispatchArgsBuffer[2] = 1;
	RWIndirectDispatchArgsBuffer[3] = PointCount;	

	RWIndirectDispatchArgsGlobalBuffer[InstanceRegisteredIndex * 4 + 0] = DispatchCount;
	RWIndirectDispatchArgsGlobalBuffer[InstanceRegisteredIndex * 4 + 1] = 1;
	RWIndirectDispatchArgsGlobalBuffer[InstanceRegisteredIndex * 4 + 2] = 1;
	RWIndirectDispatchArgsGlobalBuffer[InstanceRegisteredIndex * 4 + 3] = PointCount;
}

#endif // SHADER_CLUSTER_CULL_ARGS

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair clusters update 

#if SHADER_CLUSTERAABB

uint InstanceRegisteredIndex;
uint CurveCount;
uint ClusterOffset;
uint ClusterCount;
float ClusterScale;
float3 CPUBoundMin;
float3 CPUBoundMax;
float LODIndex;
float4x4 LocalToTranslatedWorldMatrix;

ByteAddressBuffer RenCurveBuffer;
Buffer<uint> RenPointLODBuffer;
ByteAddressBuffer RenderDeformedPositionBuffer;
StructuredBuffer<float4> RenderDeformedOffsetBuffer;

RWBuffer<int> OutClusterAABBBuffer; 	// Cluster data packed as {uint3 AABBMin, uint3 AABBMax}
RWBuffer<int> OutGroupAABBBuffer; 		// Group data packed as {uint3 AABBMin, uint3 AABBMax}

groupshared float3 SharedClusterAABBMin[GROUP_SIZE];
groupshared float3 SharedClusterAABBMax[GROUP_SIZE];
groupshared uint CurvePointIndex;
groupshared uint CurvePointCount;
groupshared float3 OutHairPositionOffset;

[numthreads(GROUP_SIZE, 1, 1)]
	void ClusterAABBEvaluationCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID, uint ThreadIndex : SV_GroupIndex)
{
#if PERMUTATION_USE_CPU_AABB
	// Only update the full group AABB
	if (all(DispatchThreadId == 0))
	{
		FHairAABB Bound;
		Bound.Min = CPUBoundMin;
		Bound.Max = CPUBoundMax;
		WriteHairAABB(InstanceRegisteredIndex, Bound, OutGroupAABBBuffer);
	}
#else // PERMUTATION_USE_CPU_AABB

	// Use the first K curves to to compute cluster bound. This is an approximation, 
	// but since curves are now shuffled, it covers most of the groom
	const uint ClusterIndex = GroupId.x;
	if (ThreadIndex == 0)
	{
		CurvePointIndex = 0;
		CurvePointCount = 0;

		const uint CurveIndex = GroupId.x;
		if (CurveIndex < CurveCount)
		{
			const FHairCurve RenCurve = ReadHairCurve(RenCurveBuffer, CurveIndex);
			CurvePointIndex = RenCurve.PointIndex;
			CurvePointCount = RenCurve.PointCount;
		}
		OutHairPositionOffset = ReadRenPositionOffset(RenderDeformedOffsetBuffer, InstanceRegisteredIndex);
	}
	GroupMemoryBarrierWithGroupSync();

	int3 ClusterAABBMin = 9999999.0f;
	int3 ClusterAABBMax =-9999999.0f;

	const uint LoopCount = DivideAndRoundUp(CurvePointCount, GROUP_SIZE);
	for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
	{
		const uint LocalPointIndex = ThreadIndex + LoopIt * GROUP_SIZE;
		if (LocalPointIndex < CurvePointCount)
		{
			const uint GlobalPointIndex = CurvePointIndex + LocalPointIndex;
			const bool bIsActive = IsHairControlPointActive(GlobalPointIndex, RenPointLODBuffer, LODIndex);
			const FHairControlPoint CP = ReadHairControlPoint(RenderDeformedPositionBuffer, GlobalPointIndex, OutHairPositionOffset, 1, 1, 1);
			if (bIsActive && all(IsFinite(CP.Position)))
			{
				const int3 WorldSpaceCentimeters = int3(mul(float4(CP.Position, 1.0f), LocalToTranslatedWorldMatrix).xyz);
				const int3 WorldSpaceCentimetersMin = WorldSpaceCentimeters - 1;
				const int3 WorldSpaceCentimetersMax = WorldSpaceCentimeters + 1;

				ClusterAABBMin = min(ClusterAABBMin, WorldSpaceCentimetersMin);
				ClusterAABBMax = max(ClusterAABBMax, WorldSpaceCentimetersMax);
			}
		}
	}
	// Scaling
	const float3 ClusterAABBCenter = float3(ClusterAABBMin + ClusterAABBMax) * 0.5f;
	const float3 ClusterAABBExtent = float3(ClusterAABBMax - ClusterAABBMin) * 0.5f;
	ClusterAABBMin = ClusterAABBCenter - ClusterAABBExtent * ClusterScale;
	ClusterAABBMax = ClusterAABBCenter + ClusterAABBExtent * ClusterScale;

	// Write each thread result to shared memory
	SharedClusterAABBMin[ThreadIndex] = ClusterAABBMin;
	SharedClusterAABBMax[ThreadIndex] = ClusterAABBMax;

	// Do a local reduction in shared memory. Assumes ClusterLOD0.VertexCount>64 to have all min&max values valid.
	GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 32)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 32]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 32]);
	}
	GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 16)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 16]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 16]);
	}
	GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 8)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 8]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 8]);
	}
	// No hardware has SIMD Vector unit operating in sync with less than 16 threads per group. So can skip sync now.
	//GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 4)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 4]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 4]);
	}
	//GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 2)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 2]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 2]);
	}
	//GroupMemoryBarrierWithGroupSync();
	if (ThreadIndex < 1)
	{
		SharedClusterAABBMin[ThreadIndex] = min(SharedClusterAABBMin[ThreadIndex], SharedClusterAABBMin[ThreadIndex + 1]);
		SharedClusterAABBMax[ThreadIndex] = max(SharedClusterAABBMax[ThreadIndex], SharedClusterAABBMax[ThreadIndex + 1]);
	}

	// Write out hair cluster AABB
	if (ThreadIndex == 0)
	{
		const uint ClusterIdx6 = (ClusterOffset + ClusterIndex) * 6;
		OutClusterAABBBuffer[ClusterIdx6 + 0] = SharedClusterAABBMin[0].x;
		OutClusterAABBBuffer[ClusterIdx6 + 1] = SharedClusterAABBMin[0].y;
		OutClusterAABBBuffer[ClusterIdx6 + 2] = SharedClusterAABBMin[0].z;
		OutClusterAABBBuffer[ClusterIdx6 + 3] = SharedClusterAABBMax[0].x;
		OutClusterAABBBuffer[ClusterIdx6 + 4] = SharedClusterAABBMax[0].y;
		OutClusterAABBBuffer[ClusterIdx6 + 5] = SharedClusterAABBMax[0].z;

		// And contribute to the group full AABB
		FHairAABB Bound;
		Bound.Min.x = SharedClusterAABBMin[0].x;
		Bound.Min.y = SharedClusterAABBMin[0].y;
		Bound.Min.z = SharedClusterAABBMin[0].z;
		Bound.Max.x = SharedClusterAABBMax[0].x;
		Bound.Max.y = SharedClusterAABBMax[0].y;
		Bound.Max.z = SharedClusterAABBMax[0].z;
		InterlockHairAABB(InstanceRegisteredIndex, Bound, OutGroupAABBBuffer);
	}

#if DEBUG_ENABLE
	if (ThreadIndex == 0)
	{
		const float4 DebugColor = float4(ColorMapViridis(ClusterIndex / float(ClusterCount)),1);
		AddAABBTWS(SharedClusterAABBMin[0], SharedClusterAABBMax[0], DebugColor);
		AddReferentialWS(LocalToWorldMatrix, 10);
	}
#endif

	// Min/Max group AABB is done later by another pass
#endif // PERMUTATION_USE_CPU_AABB
}

#endif // SHADER_CLUSTERAABB
