// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FHairAverageEnergy
{
	float3 A_R;
	float3 A_TT;
	float3 A_TRT;
};

struct FHairAverageScattering
{
	float3 A_front;	// Front Scattering A_f
	float3 A_back;	// Back Scattering A_b
};

float3 FromLinearAbsorption(float3 In) { return sqrt(In);  }
float3 ToLinearAbsorption(float3 In) { return In*In; }

FHairAverageScattering SampleHairLUT(Texture3D<float4> LUTTexture, SamplerState LUTSampler, float3 InAbsorption, float Roughness, float SinViewAngle)
{
	const float3 RemappedAbsorption = FromLinearAbsorption(InAbsorption);
	const float2 LUTValue_R = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.x)), 0).xy;
	const float2 LUTValue_G = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.y)), 0).xy;
	const float2 LUTValue_B = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.z)), 0).xy;

	FHairAverageScattering Output;
	Output.A_front = float3(LUTValue_R.x, LUTValue_G.x, LUTValue_B.x);
	Output.A_back  = float3(LUTValue_R.y, LUTValue_G.y, LUTValue_B.y);
	return Output;
}

FHairAverageEnergy SampleHairEnergyLUT(Texture3D<float4> LUTTexture, SamplerState LUTSampler, float3 InAbsorption, float Roughness, float SinViewAngle)
{
	const float3 RemappedAbsorption = FromLinearAbsorption(InAbsorption);
	const float3 LUTValue_R = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.x)), 0).xyz;
	const float3 LUTValue_G = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.y)), 0).xyz;
	const float3 LUTValue_B = LUTTexture.SampleLevel(LUTSampler, float3(saturate(abs(SinViewAngle)), saturate(Roughness), saturate(RemappedAbsorption.z)), 0).xyz;

	FHairAverageEnergy Output;
	Output.A_R = float3(LUTValue_R.x, LUTValue_G.x, LUTValue_B.x);
	Output.A_TT = float3(LUTValue_R.y, LUTValue_G.y, LUTValue_B.y);
	Output.A_TRT = float3(LUTValue_R.z, LUTValue_G.z, LUTValue_B.z);
	return Output;
}
