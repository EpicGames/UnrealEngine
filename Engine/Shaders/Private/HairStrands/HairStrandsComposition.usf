// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "HairStrandsVisibilityCommon.ush"   

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_FASTRESOLVE_MASK

Texture2D<uint> ResolveMaskTexture;

void FastResolvePS(in FScreenVertexOutput Input)
{
	const uint2 PixelCoord = floor(Input.Position.xy);
	const bool bNeedFastResolve = ResolveMaskTexture.Load(uint3(PixelCoord, 0)) > 0;

	if (!bNeedFastResolve)
		discard;
}

#endif // SHADER_FASTRESOLVE_MASK

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_COMPOSE_SAMPLE || SHADER_HOLDOUT

#include "../HeightFogCommon.ush"
#include "../SkyAtmosphereCommon.ush"

float4 EvaluateVolumetric(float3 WorldPosition, float SceneDepth)
{
	const float3 CameraRelative_WorldPosition = WorldPosition - DFHackToFloat(PrimaryView.WorldCameraOrigin);
	float4 HeightFogging = CalculateHeightFog(CameraRelative_WorldPosition);
	float4 Fogging = HeightFogging;

	if (FogStruct.ApplyVolumetricFog > 0)
	{
		const uint EyeIndex = 0;
		float3 VolumeUV = ComputeVolumeUV_DEPRECATED(WorldPosition, DFHackToFloat(PrimaryView.WorldToClip));
		Fogging = CombineVolumetricFog(HeightFogging, VolumeUV, EyeIndex, SceneDepth);
	}

	if (View.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f)
	{
		float4 NDCPosition = mul(float4(WorldPosition.xyz, 1), DFHackToFloat(PrimaryView.WorldToClip));

		// Sample the aerial perspective (AP).
		Fogging = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			View.RealTimeReflectionCapture, 
			View.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			NDCPosition, 
			CameraRelative_WorldPosition * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, 
			View.CameraAerialPerspectiveVolumeSampler,
			View.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			View.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			View.SkyAtmosphereAerialPerspectiveStartDepthKm,
			View.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			View.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			View.OneOverPreExposure, 
			Fogging);
	}

	Fogging.rgb *= View.PreExposure;

	return Fogging;
}
#endif // SHADER_COMPOSE_SAMPLE || SHADER_HOLDOUT

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_COMPOSE_SAMPLE

int2 OutputResolution;
uint bComposeDofDepth;
uint bHasHoldout;

#if PERMUTATION_READ_SLW_DEPTH
#define SLW_COMPOSE_NONE 0
#define SLW_COMPOSE_BEHIND_WATER 1
#define SLW_COMPOSE_INFRONTOF_WATER 2
uint SLWComposeType;
Texture2D<float> SLWDepthTexture;
#endif

Texture2D<float>	HairDOFDepthTexture;
Texture2D<float4>	HairLightingSampleBuffer;

void ComposeSamplePS(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	#if PERMUTATION_WRITE_DEPTH
	,out float  OutDepth : SV_DEPTH
	#endif
)
#if PERMUTATION_DEBUG == 0
{
	OutColor = 0.0f;
	const uint3 PixelCoord = uint3(floor(Input.Position.xy), 0);
	
	const float HairDeviceZ = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(PixelCoord));
	if (NodeDesc.Count == 0)
	{
		discard;
	}

	const float PixelCoverage = min(HairStrands.HairCoverageTexture.Load(PixelCoord), 1);
	if (PixelCoverage == 0)
	{
		discard;
	}

	#if PERMUTATION_READ_SLW_DEPTH
	const float SLWDepth = SLWDepthTexture.Load(PixelCoord);
	#endif
	const uint TotalNodeCount = HairStrands.HairSampleCount[0];
	const uint2 Resolution = GetHairSampleResolution(TotalNodeCount);

	const float3 ClosetPointWorldPosition = DFHackToFloat(SvPositionToWorld(float4(Input.Position.xy, HairDeviceZ, 1.0)));
	const float4 Volumetric = EvaluateVolumetric(ClosetPointWorldPosition, ConvertFromDeviceZ(HairDeviceZ));

	bool bSkipComposition = false;
	bool bHoldout = false;
	float3 LocalAccColor = 0;
	LOOP
	for (uint SampleIt = 0; SampleIt < NodeDesc.Count; SampleIt++)
	{
		#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
		if (bHasHoldout)
		{
			const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[NodeDesc.Offset + SampleIt]);
			bHoldout = bHoldout || HasHairFlags(Sample.Flags, HAIR_FLAGS_HOLDOUT);
		}
		#endif
		
		#if PERMUTATION_READ_SLW_DEPTH
		{
			const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[NodeDesc.Offset + SampleIt]);
			if (SLWComposeType == SLW_COMPOSE_BEHIND_WATER)
			{
				bSkipComposition = bSkipComposition || Sample.Depth >= SLWDepth;
			}
			else if (SLWComposeType == SLW_COMPOSE_INFRONTOF_WATER)
			{
				bSkipComposition = bSkipComposition || Sample.Depth < SLWDepth;
			}
		}
		#endif

		const uint LinearCoord = NodeDesc.Offset + SampleIt;
		const uint2 Coord = GetHairSampleCoord(LinearCoord, Resolution);
		LocalAccColor += HairLightingSampleBuffer.Load(uint3(Coord, 0)).xyz;
	}

	#if PERMUTATION_READ_SLW_DEPTH
	if (bSkipComposition)
	{
		discard;
	}

	#endif

	// If any sample support holdout, then the lighting contribution is set to 0 to mask out the current hair pixel 
	// but fog from camera to hair pixel needs to be accounted for
	if (bHoldout)
	{
		LocalAccColor = 0;
	}
	OutColor.rgb = (LocalAccColor * Volumetric.a + Volumetric.rgb) * PixelCoverage;
	OutColor.a = PixelCoverage;

	#if PERMUTATION_WRITE_DEPTH
	OutDepth = HairDeviceZ;
	if (bComposeDofDepth)
	{
		OutDepth = HairDOFDepthTexture.Load(PixelCoord);
	}
	#endif
}
#else // PERMUTATION_DEBUG == 1
{
	OutColor = 0.0f;
	const uint3 PixelCoord = uint3(floor(Input.Position.xy), 0);

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(PixelCoord));
	if (NodeDesc.Count == 0)
	{
		discard;
	}

	const float PixelCoverage = min(HairStrands.HairCoverageTexture.Load(PixelCoord), 1);
	if (PixelCoverage == 0)
		discard;

	const float HairDeviceZ = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;

	float3 LocalAccColor = 0;
	LOOP
	for (uint SampleIt = 0; SampleIt < NodeDesc.Count; SampleIt++)
	{
		const FPackedHairSample PackedSample = HairStrands.HairSampleData[NodeDesc.Offset + SampleIt];
		const FHairSample Sample = UnpackHairSample(PackedSample);
		const float LocalCoverage = From8bitCoverage(Sample.Coverage8bit);
		LocalAccColor += LocalCoverage * Sample.BaseColor;
	}
	OutColor.rgb = LocalAccColor;

	OutColor.rgb *= PixelCoverage;
	OutColor.a = PixelCoverage;
	#if PERMUTATION_WRITE_DEPTH
	OutDepth = HairDeviceZ;
	#endif
}
#endif

#endif // SHADER_COMPOSE_SAMPLE

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_HOLDOUT

// PassType:
// * 0: Output attenuation for the scene pixel (alpha only)
// * 1: Output hair contribution to the scene pixel (alpha value only)
uint PassType;
uint bComposeDofDepth;
Texture2D<float> HairDOFDepthTexture;

void HoldoutPS(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0,
	out float  OutDepth : SV_DEPTH)
{
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	OutColor = 0.0f;
	const uint3 PixelCoord = uint3(floor(Input.Position.xy), 0);
	
	const float HairDeviceZ = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(PixelCoord));
	if (NodeDesc.Count == 0)
	{
		discard;
	}

	const float PixelCoverage = min(HairStrands.HairCoverageTexture.Load(PixelCoord), 1);
	if (PixelCoverage == 0)
	{
		discard;
	}

	bool bHoldout = false;

	LOOP
	for (uint SampleIt = 0; SampleIt < NodeDesc.Count; SampleIt++)
	{
		const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[NodeDesc.Offset + SampleIt]);
		bHoldout = bHoldout || HasHairFlags(Sample.Flags, HAIR_FLAGS_HOLDOUT);
	}

	const float3 ClosetPointWorldPosition = DFHackToFloat(SvPositionToWorld(float4(Input.Position.xy, HairDeviceZ, 1.0)));
	const float4 Volumetric = EvaluateVolumetric(ClosetPointWorldPosition, ConvertFromDeviceZ(HairDeviceZ));

	if (!bHoldout)
	{
		discard;
	}

	//OutColor.rgb = (LocalAccColor * Volumetric.a + Volumetric.rgb) * PixelCoverage;

	OutColor.rgb = 0;
	OutColor.a   = PassType == 0 ? PixelCoverage : Volumetric.a * PixelCoverage;
	OutDepth     = HairDeviceZ;
	if (bComposeDofDepth)
	{
		OutDepth = HairDOFDepthTexture.Load(PixelCoord);
	}
#else
	OutColor = 0;
	OutDepth = 0;
	discard;
#endif
}

#endif // SHADER_HOLDOUT

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_DOFDEPTH
#include "../CircleDOFCommon.ush"

// Included from DeferredLightingCommon now
//Texture2D<float>	SceneDepthTexture;

void DOFDepthPS(
	in FScreenVertexOutput Input,
	out float OutTexture : SV_Target0)
{
	OutTexture = 0.0f;
	const uint3 PixelCoord = uint3(floor(Input.Position.xy), 0);

	const float PixelCoverage = min(HairStrands.HairCoverageTexture.Load(PixelCoord), 1);
	if (PixelCoverage == 0)
	{
		return;
	}
	   
	const float HalfResToFullRes = 2;

	float SceneDeviceZ = 0;
	float SceneDepth = 0;
	float SceneCoC = 0; // Radius in pixel
	{
		SceneDeviceZ = SceneDepthTexture.Load(PixelCoord).x;
		SceneDepth = ConvertFromDeviceZ(SceneDeviceZ);
		SceneDeviceZ = 1 - SceneDeviceZ;
		SceneCoC = DepthToCoc(SceneDepth) * HalfResToFullRes;
	}

	float HairLuminance = 0;
	float HairDeviceZ = 0;
	float HairDepth = 0;
	float HairCoC = 0; // Radius in pixel
	{
		HairDeviceZ = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;
		HairDepth = ConvertFromDeviceZ(HairDeviceZ);
		HairDeviceZ = 1 - HairDeviceZ;
		HairCoC = DepthToCoc(HairDepth) * HalfResToFullRes;
	}

	const float OutDeviceZ = lerp(SceneDeviceZ, HairDeviceZ, PixelCoverage);
	OutTexture = 1 - OutDeviceZ;
}

#endif // SHADER_DOFDEPTH

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_WRITE_GBUFFER

#include "HairStrandsGBufferCommon.ush"

#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
#define SUBSTRATE_INLINE_SHADING 1
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateExport.ush"
uint MaxBytesPerPixel;
RWTexture2D<SUBSTRATE_TOP_LAYER_TYPE> RWTopLayerTexture;
RWTexture2DArray<uint> RWMaterialTextureArray;
#endif

uint bWriteDummyData;

void MainPS(
	in float2 UV : TEXCOORD
	, in float4 SvPosition: SV_Position
#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
  #if PERMUTATION_OUTPUT_TYPE == 1
	, out float  OutDepth    : SV_DEPTH
  #endif
#else
	, out float4 OutGBufferA : SV_Target0
	, out float4 OutGBufferB : SV_Target1
  #if PERMUTATION_OUTPUT_TYPE == 1
	, out float4 OutGBufferC : SV_Target2
	, out float4 OutGBufferD : SV_Target3
	, out float4 OutGBufferE : SV_Target4
	, out float  OutDepth    : SV_DEPTH
  #endif
#endif
)
{
	ResolvedView = ResolveView();
#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
  #if PERMUTATION_OUTPUT_TYPE == 1
	OutDepth = 0;
  #endif
#else
	OutGBufferA = 0;
	OutGBufferB = 0;
  #if PERMUTATION_OUTPUT_TYPE == 0
	float4 OutGBufferC = 0;
	float4 OutGBufferD = 0;
	float4 OutGBufferE = 0;
	float  OutDepth = 0;
  #elif PERMUTATION_OUTPUT_TYPE == 1
	OutGBufferC = 0;
	OutGBufferD = 0;
	OutGBufferE = 0;
	OutDepth = 0;
  #endif
#endif

	const uint3 PixelCoord = uint3(floor(SvPosition.xy), 0);

	const bool bIsValid = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x > 0;
	if (!bIsValid)
	{
		discard;
	}

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(PixelCoord));
	if (NodeDesc.Count == 0)
	{
		discard;
	}

	#if SUBSTRATE_ENABLED == 0 || SUBSTRATE_GBUFFER_FORMAT==0
	if (bWriteDummyData > 0)
	{
		const float3 Tangent = float3(0, 0, 1);
		OutGBufferA = float4(Tangent, 0);
		OutGBufferB = float4(0, 0, 0, EncodeShadingModelIdAndSelectiveOutputMask(SHADINGMODELID_HAIR, 0));
	}
	else
	#endif
	{
		float3 Samples_Tangent = 0;
		float  Samples_Specular = 0;
		float3 Samples_BaseColor = 0;
		float  Samples_Roughness = 0;
		uint   Samples_LightChannelMask = 0;
		float  Samples_Backlit = 0;
		float  Samples_Depth = 0;
		LOOP
		for (uint NodeIt = 0; NodeIt < NodeDesc.Count; ++NodeIt)
		{
			const uint NodeOffset = NodeDesc.Offset + NodeIt;
			const FPackedHairSample Data = HairStrands.HairSampleData[NodeOffset];
			const FHairSample Sample = UnpackHairSample(Data);

			Samples_Tangent		+= Sample.Tangent;
			Samples_Specular	+= Sample.Specular;
			Samples_BaseColor	+= Sample.BaseColor;
			Samples_Roughness	+= Sample.Roughness;
			Samples_Backlit		+= Sample.Backlit;
			if (NodeIt == 0)
			{
				Samples_LightChannelMask = Sample.LightChannelMask;
			}
		}
		Samples_Depth = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;

		const float InvSampleCount = 1.f / max(1u, NodeDesc.Count);
		Samples_Tangent		 = normalize(Samples_Tangent);
		Samples_Specular	*= InvSampleCount;
		Samples_BaseColor	*= InvSampleCount;
		Samples_Roughness	*= InvSampleCount;
		Samples_Backlit		*= InvSampleCount;

	#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
		const bool bWriteHairData = bWriteDummyData == 0;
		FSubstrateBSDF HairBSDF = GetSubstrateHairBSDF(Samples_BaseColor, 0.f /*Scatter*/, Samples_Specular, Samples_Roughness, Samples_Backlit, 0 /*Emissive*/, true /*bComplexTransmittance*/, 0/*SharedLocalBasisIndex*/).InlinedBSDF;

		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord.xy, uint2(ResolvedView.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = InitialiseSubstratePixelHeader();
		SubstratePixelHeader.ClosureCount = 1;
		SubstratePixelHeader.SharedLocalBases.Normals[0] = normalize(Samples_Tangent);

		FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
		FRWSubstrateMaterialContainer RWSubstrateMaterialContainer = InitialiseRWSubstrateMaterialContainer(RWMaterialTextureArray /*Should normally be WithoutRTs, but here writes are explicit*/, 1/*QuadPixelWriteMask*/, PixelCoord.xy);
		WriteHair(
			SubstratePixelHeader,
			SubstrateAddressing,
			HairBSDF,
			bWriteHairData,
			TopLayerData,
			RWSubstrateMaterialContainer,
			RWMaterialTextureArray);

		RWTopLayerTexture[SubstrateAddressing.PixelCoords] = SubstratePackTopLayerData(TopLayerData);
		RWMaterialTextureArray[uint3(SubstrateAddressing.PixelCoords,0)] = RWSubstrateMaterialContainer.MaterialRenderTargets[0]; // Header
		if (bWriteHairData)
		{
			RWMaterialTextureArray[uint3(SubstrateAddressing.PixelCoords,1)] = RWSubstrateMaterialContainer.MaterialRenderTargets[1];
			RWMaterialTextureArray[uint3(SubstrateAddressing.PixelCoords,2)] = RWSubstrateMaterialContainer.MaterialRenderTargets[2];
		}

		#if PERMUTATION_OUTPUT_TYPE == 1
		OutDepth = Samples_Depth;
		#endif

	#else
		WriteGBuffer(
			Samples_Tangent,
			Samples_Specular,
			Samples_BaseColor,
			Samples_Roughness,
			Samples_LightChannelMask,
			Samples_Backlit,
			Samples_Depth,
			OutGBufferA,
			OutGBufferB,
			OutGBufferC,
			OutGBufferD,
			OutGBufferE,
			OutDepth);
	#endif
	}
}

#endif // SHADER_WRITE_GBUFFER