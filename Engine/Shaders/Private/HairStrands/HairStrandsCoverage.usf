// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"

int2 OutputResolution;
float LUT_HairCount; 
float LUT_HairRadiusCount;

SamplerState LinearSampler;
Texture2D<float> HairCoverageLUT;
#if PERMUTATION_INPUT_TYPE == 0
	Texture2D<float2> HairCountTexture;
#elif PERMUTATION_INPUT_TYPE == 1
	Texture2D<uint> HairCountTexture;
#endif
RWTexture2D<float> OutputTexture;

#define TILE_PIXEL_SIZE 8

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
#if PERMUTATION_INPUT_TYPE == 0
{
	const int2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord > OutputResolution))
	{
		return;
	}

	const float2 Raw = HairCountTexture.Load(uint3(PixelCoord, 0)).xy;

	const float RemapScale = 2; // Remap the radius from [0..0.5] -> [0..1], i.e. the mapping used by the LUT

	const float HairCount = Raw.y;
	const float HairRadius = saturate(Raw.x / HairCount * RemapScale);
	const float2 UV = float2(HairRadius, (HairCount+0.5f) / LUT_HairCount);
	const float Coverage = HairCoverageLUT.SampleLevel(LinearSampler, UV, 0);
	
	OutputTexture[PixelCoord] = saturate(1-Coverage);
}
#endif
#if PERMUTATION_INPUT_TYPE == 1
{
	const int2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord > OutputResolution))
	{
		return;
	}

	const float PrecisionScale = 1000.f;
	const uint PackedHairViewTransmittance = HairCountTexture.Load(uint3(PixelCoord, 0));
	const float LogViewTransmittance = -float(PackedHairViewTransmittance) / PrecisionScale;
	const float ViewTransmittance = exp2(LogViewTransmittance);
	OutputTexture[PixelCoord] = ViewTransmittance;
}
#endif