// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush" 
#include "/Engine/Shared/HairStrandsDefinitions.h"
#include "../HairShadingCommon.ush"
#include "../ColorMap.ush"

#if SHADER_PRINT_INSTANCE || SHADER_PRINT || SHADER_PRINT_MEMORY || SHADER_CLUSTERAABB || SHADER_DOMINFO || SHADER_PRINT_HAIR_SKIN_CACHE
/////////////////////////////////////////////////////////////////////////////////////////
// Utils function

#include "../ShaderPrint.ush"

// Shared color
float4 GetGPUSceneBoundColor()						{ return ColorDarkGreen; }
float4 GetInstanceGroupScreenBoundColor()			{ return ColorGreen; }

float4 GetGPUAABBInstanceGroupBoundColor()			{ return ColorDarkOrange; }
float4 GetGPUAABBMacroGroupBoundColor()				{ return ColorRed; }
float4 GetGPUAABBMacroGroupBoundVoxelAlignedColor()	{ return ColorLightRed; }

// Print float number in 0..1000 aligned on the right
void PrintNumberAlignedRight(inout FShaderPrintContext Context, float Value, FFontColor InColor)
{
	if (Value >= 100.f)
	{
		Print(Context, Value, InColor, 5, 1);
	}
	else if (Value >= 10.f)
	{
		Print(Context, TEXT(" "), InColor);
		Print(Context, Value, InColor, 4, 1);
	}
	else
	{
		Print(Context, TEXT("  "), InColor);
		Print(Context, Value, InColor, 3, 1);
	}
}

// Print 
void PrintLargeNumber(inout FShaderPrintContext Context, uint Value, FFontColor InColor)
{
	if (Value > 1000000000)
	{
		const float fValue = Value / 1000000000.f;
		//Print(Context, fValue, InColor, 5, 1);
		PrintNumberAlignedRight(Context, fValue, InColor);
		Print(Context, TEXT("B"), InColor);
	}
	else if (Value > 1000000)
	{
		const float fValue = Value / 1000000.f;
		//Print(Context, fValue, InColor, 5, 1);
		PrintNumberAlignedRight(Context, fValue, InColor);
		Print(Context, TEXT("M"), InColor);
	}
	else if (Value > 1000)
	{
		const float fValue = Value / 1000.f;
		//Print(Context, fValue, InColor, 5, 1);
		PrintNumberAlignedRight(Context, fValue, InColor);
		Print(Context, TEXT("k"), InColor);
	}
	else
	{
		Print(Context, TEXT("   "), InColor);
		Print(Context, Value, InColor, 3);
	}
}

void PrintPercentage(inout FShaderPrintContext Context, float Value, FFontColor InColor)
{
	const uint iValue = ceil(clamp(Value * 100.f, 0.f, 100.f));
	if (iValue < 10)
	{
		Print(Context, TEXT("  "), InColor);
		Print(Context, iValue, InColor, 1);
	}
	else if (iValue < 100)
	{
		Print(Context, TEXT(" "), InColor);
		Print(Context, iValue, InColor, 2);
	}
	else
	{
		Print(Context, iValue, InColor, 3);
	}
	Print(Context, TEXT("% "), InColor);
}

#endif // SHADER_PRINT_INSTANCE || SHADER_PRINT || SHADER_PRINT_MEMORY || SHADER_CLUSTERAABB || SHADER_DOMINFO || SHADER_PRINT_HAIR_SKIN_CACHE

/////////////////////////////////////////////////////////////////////////////////////////
// Visualize hair strands data as full screen view mode (BaseColor, Tangent, Roughness, ...)

#if SHADER_DEBUG_MODE

#include "../ColorMap.ush"
#include "HairStrandsVisibilityCommon.ush"

float2 OutputResolution;
uint FastResolveMask;
uint DebugMode;
int SampleIndex;

Texture2D<uint2> DepthStencilTexture;
SamplerState LinearSampler;

#define DEBUG_MODE_SAMPLE_COUNT 0 
#define DEBUG_MODE_COVERAGE_TYPE 1
#define DEBUG_MODE_TAA_RESOLVE_TYPE 2
#define DEBUG_MODE_COVERAGE 3
#define DEBUG_MODE_DEPTH 4
#define DEBUG_MODE_BASECOLOR 5
#define DEBUG_MODE_ROUGHNESS 6
#define DEBUG_MODE_SPECULAR 7
#define DEBUG_MODE_TANGENT 8
#define DEBUG_MODE_TILE 9

void MainPS(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	const float2 UV = Input.UV;
	const float2 PixelPos = UV * OutputResolution;

	const float HairPixelCoverage = HairStrands.HairCoverageTexture.Load(uint3(PixelPos, 0));
	const bool bIsValid = HairPixelCoverage > 0 || DebugMode == DEBUG_MODE_TILE;
	if (!bIsValid)
		discard;

	const bool bIsDepthValid = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelPos, 0)).x > 0;
	if (!bIsDepthValid)
	{
		discard;
	}

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(uint3(PixelPos, 0)));

	float3 Color = 0;
	float Alpha = 1;
	if (DebugMode == DEBUG_MODE_SAMPLE_COUNT)
	{
		Color = GetHSVDebugColor(NodeDesc.Count/float(HairStrands.MaxSamplePerPixelCount));
	}

	if (DebugMode == DEBUG_MODE_COVERAGE_TYPE)
	{
		const bool bIsFullyCovered = HairPixelCoverage >= 1;
		Color = bIsFullyCovered ? float3(0.25,0.85,0.25) : float3(0.85,0.25,0.25);
	}

	if (DebugMode == DEBUG_MODE_TAA_RESOLVE_TYPE)
	{
		const uint Stencil = DepthStencilTexture.Load(uint3(PixelPos,0)).y;
		const bool bIsFastResolve = (Stencil & FastResolveMask) != 0;
		Color = bIsFastResolve ? float3(1,1,0) : float3(0.15f, 0.25f, 0.75f);
	}

	if (DebugMode == DEBUG_MODE_COVERAGE)
	{
		Color = GetHSVDebugColor(saturate(HairPixelCoverage));
	}

	const bool bIsDebugModeMaterial =
		DebugMode == DEBUG_MODE_DEPTH ||
		DebugMode == DEBUG_MODE_BASECOLOR ||
		DebugMode == DEBUG_MODE_ROUGHNESS ||
		DebugMode == DEBUG_MODE_SPECULAR || 
		DebugMode == DEBUG_MODE_TANGENT;

	if (NodeDesc.Count > 0 && bIsDebugModeMaterial)
	{
		const int LocalSampleIndex = SampleIndex; 
		const bool bAverageAllSamples = LocalSampleIndex < 0;
		const uint SelectedIndex = LocalSampleIndex >= 0 ? clamp(LocalSampleIndex, 0, int(NodeDesc.Count - 1)) : 0;
		float	Depth = 0;
		float3	Tangent = 0;
		float3	BaseColor = 0;
		float	Roughness = 0;
		float	Specular = 0;

		for (uint NodeIt = 0; NodeIt < NodeDesc.Count; ++NodeIt)
		{
			const uint NodeOffset = NodeDesc.Offset + NodeIt;
			const FPackedHairSample Data = HairStrands.HairSampleData[NodeOffset];
			const FHairSample Sample = UnpackHairSample(Data);

			if (bAverageAllSamples)
			{
				Depth		+= Sample.Depth;
				Tangent		+= Sample.Tangent;
				BaseColor	+= Sample.BaseColor;
				Roughness	+= Sample.Roughness;
				Specular	+= Sample.Specular;
			}
			else if (SelectedIndex == NodeIt)
			{
				Depth		= Sample.Depth;
				Tangent		= Sample.Tangent;
				BaseColor	= Sample.BaseColor;
				Roughness	= Sample.Roughness;
				Specular	= Sample.Specular;
			}
		}

		if (bAverageAllSamples)
		{
			Depth		/= NodeDesc.Count;
			Tangent		/= NodeDesc.Count;
			BaseColor	/= NodeDesc.Count;
			Roughness	/= NodeDesc.Count;
			Specular	/= NodeDesc.Count;
		}

		if (DebugMode == DEBUG_MODE_DEPTH)
		{
			Color = Depth;
		}

		if (DebugMode == DEBUG_MODE_BASECOLOR)
		{
			Color = BaseColor;
		}

		if (DebugMode == DEBUG_MODE_ROUGHNESS)
		{
			Color = Roughness;
		}

		if (DebugMode == DEBUG_MODE_SPECULAR)
		{
			Color = Specular;
		}

		if (DebugMode == DEBUG_MODE_TANGENT)
		{
			Color = (Tangent+1) * 0.5f;
		}
	}

	OutColor = float4(Color, Alpha);
}

#endif // SHADER_DEBUG_MODE

/////////////////////////////////////////////////////////////////////////////////////////
// Visualize hair strands control points

#if SHADER_CVS
#include "HairStrandsVertexFactoryCommon.ush"
#include "../ShaderPrint.ush"

float4x4 LocalToWorld;
uint MaxVertexCount;
uint InstanceIt;
float3 GroupColor;

SamplerState LinearSampler;
Texture2D<float> DepthTexture;
RWTexture2D<float4> ColorTexture;

[numthreads(256, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint VertexId = DispatchThreadId.x;
	if (VertexId > MaxVertexCount)
		return;

	uint VertexIndex = VertexId;
	if (HairStrandsVF.bCullingEnable)
	{
		VertexIndex = HairStrandsVF.CullingIndexBuffer[VertexIndex];
	}

	const float3 HairStrandsVF_RenPositionOffset = ReadRenPositionOffset(HairStrandsVF.PositionOffsetBuffer, HairStrandsVF.RegisteredIndex);

	FShaderPrintContext CtxWidget = InitShaderPrintContext(InstanceIt == 0 && VertexId == 0, uint2(50, 50));
	Print(CtxWidget, TEXT("Controls"), FontOrange); Newline(CtxWidget);
	const bool bOffsetPosition = AddCheckbox(CtxWidget, TEXT("Position Offset"), false, FontWhite); Newline(CtxWidget);
	const bool bControlPoints  = AddCheckbox(CtxWidget, TEXT("Control Points"), true, FontWhite); Newline(CtxWidget);
	const bool bGroupColor     = AddCheckbox(CtxWidget, TEXT("Group Color"), true, FontWhite); Newline(CtxWidget);
	const bool bLargePoint     = AddCheckbox(CtxWidget, TEXT("Large Point"), false, FontWhite); Newline(CtxWidget);
//	int PointSize              = AddSlider(CtxWidget,   TEXT("Point Size"), 1, FontWhite, 1, 3); Print(CtxWidget, PointSize, FontYellow); Newline(CtxWidget);	

	FShaderPrintContext Ctx = InitShaderPrintContext(VertexId == 0, uint2(50, 50));
	if (Ctx.bIsActive && bOffsetPosition)
	{
		const float3 P_Local = HairStrandsVF_RenPositionOffset;
		const float3 P_World = mul(float4(P_Local, 1), LocalToWorld).xyz;
		AddCrossWS(P_World, 10.f, ColorRed);
	}

	if (bControlPoints)
	{
		const float3 P_Local = ReadHairControlPoint(
			HairStrandsVF.PositionBuffer,
			VertexIndex,
			HairStrandsVF_RenPositionOffset,
			HairStrandsVF.Radius,
			HairStrandsVF.RootScale,
			HairStrandsVF.TipScale).Position;

		const float3 P_World = mul(float4(P_Local, 1), LocalToWorld).xyz;
		float4 SamplePosition = float4(P_World + DFHackToFloat(PrimaryView.PreViewTranslation), 1);
		float4 ClipPosition = mul(SamplePosition, View.TranslatedWorldToClip);
		float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;
		float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
		float SampleSceneDepth = ClipPosition.w;

		const float S = 1;
		const float4 Color = bGroupColor ? float4(GroupColor, 1) : float4(S, S, 0, 1);
		const float SceneDepth = ConvertFromDeviceZ(DepthTexture.SampleLevel(LinearSampler, ScreenUV, 0));
		const bool bIsVisible = SampleSceneDepth <= SceneDepth;
		if (bIsVisible)
		{
			int2 OutCoord = ScreenUV * View.BufferSizeAndInvSize.xy;
			OutCoord = clamp(OutCoord, int2(0, 0), View.BufferSizeAndInvSize.xy-1);

			if (bLargePoint)
			{
				int PointSize = 1;
				for (int y = -PointSize; y <= PointSize; ++y)
				for (int x = -PointSize; x <= PointSize; ++x)
				{
					int2 SplatCoord = OutCoord + int2(x, y);
					SplatCoord = clamp(SplatCoord, int2(0, 0), View.BufferSizeAndInvSize.xy-1);
					ColorTexture[SplatCoord] = Color;
				}
			}
			else
			{
				ColorTexture[OutCoord] = Color;
			}
		}
	}
}

#endif // SHADER_CVS


/////////////////////////////////////////////////////////////////////////////////////////
// Visualize (Sim/Render & Rest/Deformed):
// * hair strands root points
// * skin triangle on which strands are attached to
// * sample point used for RBF

#if SHADER_MESH_PROJECTION_HAIR

#include "HairStrandsBindingCommon.ush" 
#include "../ShaderPrint.ush"

#define INPUT_ROOTS 0
#define INPUT_SAMPLES 1

uint bSimPass;
uint bRestPass;
uint DrawIndex;
uint MaxUniqueSection;
uint MaxUniqueTriangleCount;
uint MaxRootCount;
float4x4 RootLocalToWorld;
uint DeformedFrameEnable;

uint bOverride;
uint Render_HairRestTriangles;
uint Render_HairRestFrames;
uint Render_HairDeformedTriangles;
uint Render_HairDeformedFrames;
uint Sim_HairRestTriangles;
uint Sim_HairRestFrames;
uint Sim_HairRestSamples;
uint Sim_HairDeformedTriangles;
uint Sim_HairDeformedFrames;
uint Sim_HairDeformedSamples;

#if PERMUTATION_INPUT_TYPE == INPUT_ROOTS
Buffer<uint>   RootBarycentricBuffer;
Buffer<float4> RestPositionBuffer;
Buffer<float4> DeformedPositionBuffer;
Buffer<uint>   RootToUniqueTriangleIndexBuffer;
#endif

#if PERMUTATION_INPUT_TYPE == INPUT_SAMPLES
StructuredBuffer<float4> RestSamplePositionsBuffer;
StructuredBuffer<float4> DeformedSamplePositionsBuffer;
#endif

StructuredBuffer<float4> RestPositionOffset;
StructuredBuffer<float4> DeformedPositionOffset;

void Move(inout FShaderPrintContext Ctx, uint2 InPos)
{
	if (Ctx.bIsActive)
	{
		float2 fPos = float2(InPos) / float2(Ctx.Config.Resolution);
		Ctx.StartPos = fPos;
		Ctx.Pos 	 = fPos;
	}
}

float2 ToScreenUV(float3 TWSPos)
{
	const float4 NDC = mul(float4(TWSPos.xyz, 1), View.TranslatedWorldToClip);
	const float2 ScreenUv = (NDC.xy / NDC.ww) * float2(0.5f, -0.5f) + 0.5f;
	return ScreenUv;
}

float4 GetScreenMinMax(float3 WP0, float3 WP1, float3 WP2)
{
	const float3 PreViewTranslation = DFDemote(MakeDFVector3(View.PreViewTranslationHigh, View.PreViewTranslationLow));
	const float3 TWSP0 = WP0 + PreViewTranslation;
	const float3 TWSP1 = WP1 + PreViewTranslation;
	const float3 TWSP2 = WP2 + PreViewTranslation;

	const float2 UV0 = ToScreenUV(TWSP0);
	const float2 UV1 = ToScreenUV(TWSP1);
	const float2 UV2 = ToScreenUV(TWSP2);

	float2 MinUV;
	float2 MaxUV;
	MinUV.x = min3(UV0.x, UV1.x, UV2.x);
	MinUV.y = min3(UV0.y, UV1.y, UV2.y);
	MaxUV.x = max3(UV0.x, UV1.x, UV2.x);
	MaxUV.y = max3(UV0.y, UV1.y, UV2.y);
	return float4(MinUV, MaxUV);
}

bool IsInRect(float4 MinMaxUV, float2 UV)
{
	return all(UV >= MinMaxUV.xy) && all(UV <= MinMaxUV.zw);
}

[numthreads(256, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();
	float NormalScale = 0.2f;
	
	const float2 CursorPixelCoord = GetCursorPos();
	const float2 CursorUV = CursorPixelCoord * View.ViewSizeAndInvSize.zw;


	bool bDrawTriangle= false;
	bool bDrawNormal  = false;
	bool bDrawRoot    = false; 

	// Widgets for draw configuration
	{	
		uint2 StartPos  = uint2(500, 50);
		uint2 OffsetPos = uint2(180, 0);
		FShaderPrintContext Ctx = InitShaderPrintContext(DispatchThreadId.x == 0 && DrawIndex == 0, StartPos);
		bool bValid = false;

		// Guides
		{
			bool bRest    = false;
			bool bDeform  = false;

			bool bTriangle= false;
			bool bNormal  = false;
			bool bRoot    = false; 

			Print(Ctx, TEXT("Guides "), FontYellow); Newline(Ctx);
			if (bOverride>0)
			{
				bRest    = Sim_HairRestTriangles     || Sim_HairRestFrames;
				bDeform  = Sim_HairDeformedTriangles || Sim_HairDeformedFrames;
				bTriangle= Sim_HairRestTriangles     || Sim_HairDeformedTriangles;
				bNormal  = false;
				bRoot    = Sim_HairRestFrames        || Sim_HairDeformedFrames;

				Print(Ctx, TEXT("Sim Rest      "), FontWhite);	PrintBool(Ctx, bRest); 		Newline(Ctx);
				Print(Ctx, TEXT("Sim Deformed  "), FontWhite);	PrintBool(Ctx, bDeform);	Newline(Ctx);
				Newline(Ctx);
	
				Print(Ctx, TEXT("Sim Triangles "), FontWhite);	PrintBool(Ctx, bTriangle); 	Newline(Ctx);
				Print(Ctx, TEXT("Sim Normals   "), FontWhite);	PrintBool(Ctx, bNormal); 	Newline(Ctx);
				Print(Ctx, TEXT("Sim Roots     "), FontWhite); 	PrintBool(Ctx, bRoot);		Newline(Ctx);
				Newline(Ctx);
	
			}
			else
			{
				bRest     = AddCheckbox(Ctx, TEXT("Sim Rest"), false, FontWhite); Newline(Ctx);
				bDeform   = AddCheckbox(Ctx, TEXT("Sim Deformed"), false, FontWhite); Newline(Ctx);
				Newline(Ctx);
		
				bTriangle = AddCheckbox(Ctx, TEXT("Sim Triangles"), false, FontWhite); Newline(Ctx);
				bNormal   = AddCheckbox(Ctx, TEXT("Sim Normals"), false, FontWhite); Newline(Ctx);
				bRoot     = AddCheckbox(Ctx, TEXT("Sim Roots"), false, FontWhite); Newline(Ctx);
				Newline(Ctx);
			}

			if (bSimPass == 1)
			{
				if (PERMUTATION_INPUT_TYPE == INPUT_ROOTS)
				{
					bValid = (bRoot || bNormal || bTriangle)  && ((bRestPass==1 && bRest) || (bRestPass==0 && bDeform));

					bDrawTriangle = bDrawTriangle || bTriangle;
					bDrawNormal   = bDrawNormal   || bNormal;
					bDrawRoot     = bDrawRoot     || bRoot; 

				}
			}
		}
		// Strands/Cards
		{
			bool bRest    = false;
			bool bDeform  = false;

			bool bTriangle= false;
			bool bNormal  = false;
			bool bRoot    = false; 

			Move(Ctx, StartPos + OffsetPos);
			Print(Ctx, TEXT("Strands / Cards"), FontYellow); Newline(Ctx);
			if (bOverride>0)
			{
				bRest    = Render_HairRestTriangles     || Render_HairRestFrames;
				bDeform  = Render_HairDeformedTriangles || Render_HairDeformedFrames;
				bTriangle= Render_HairRestTriangles     || Render_HairDeformedTriangles;
				bNormal  = false;
				bRoot    = Render_HairRestFrames        || Render_HairDeformedFrames;
	
				Print(Ctx, TEXT("Ren Rest      "), FontWhite);	PrintBool(Ctx, bRest); 		Newline(Ctx);
				Print(Ctx, TEXT("Ren Deformed  "), FontWhite);	PrintBool(Ctx, bDeform);	Newline(Ctx);
				Newline(Ctx);
			
				Print(Ctx, TEXT("Ren Triangles "), FontWhite);	PrintBool(Ctx, bTriangle); 	Newline(Ctx);
				Print(Ctx, TEXT("Sim Normals   "), FontWhite);	PrintBool(Ctx, bNormal); 	Newline(Ctx);
				Print(Ctx, TEXT("Ren Roots     "), FontWhite); 	PrintBool(Ctx, bRoot);		Newline(Ctx);
				Newline(Ctx);
			}
			else
			{
				bRest     = AddCheckbox(Ctx, TEXT("Ren Rest"), false, FontWhite); Newline(Ctx);
				bDeform   = AddCheckbox(Ctx, TEXT("Ren Deformed"), true, FontWhite) ; Newline(Ctx);
				Newline(Ctx);
				
				bTriangle = AddCheckbox(Ctx, TEXT("Ren Triangles"), false, FontWhite); Newline(Ctx);
				bNormal   = AddCheckbox(Ctx, TEXT("Ren Normals"), false, FontWhite); Newline(Ctx);
				bRoot     = AddCheckbox(Ctx, TEXT("Ren Roots"), true, FontWhite); Newline(Ctx);
				Newline(Ctx);
			}

			if (bSimPass == 0)
			{
				if (PERMUTATION_INPUT_TYPE == INPUT_ROOTS)
				{
					bValid = (bRoot || bNormal || bTriangle) && ((bRestPass==1 && bRest) || (bRestPass==0 && bDeform));

					bDrawTriangle = bDrawTriangle || bTriangle;
					bDrawNormal   = bDrawNormal   || bNormal;
					bDrawRoot     = bDrawRoot     || bRoot; 
				}
			}
		}

		// RBF
		{
			bool bRest   = false;
			bool bDeform = false;

			Move(Ctx, StartPos + OffsetPos*2);
			Print(Ctx, TEXT("RBF Samples"), FontYellow); Newline(Ctx);
			if (bOverride>0)
			{
				bRest	= Sim_HairRestSamples;
				bDeform = Sim_HairDeformedSamples;

				Print(Ctx, TEXT("RBF Rest     "), FontWhite); Print(Ctx, TEXT("|| "), FontPurple); PrintBool(Ctx, bRest) Newline(Ctx);
				Print(Ctx, TEXT("RBF Deformed "), FontWhite); Print(Ctx, TEXT("|| "), FontYellow); PrintBool(Ctx, bDeform) Newline(Ctx);
				Newline(Ctx);
			}
			else
			{
				bRest     = AddCheckbox(Ctx, TEXT("RBF Rest    "), false, FontWhite); Print(Ctx, TEXT("||"), FontPurple); Newline(Ctx);
				bDeform   = AddCheckbox(Ctx, TEXT("RBF Deformed"), false, FontWhite); Print(Ctx, TEXT("||"), FontYellow); Newline(Ctx);
				Newline(Ctx);	
			}


			if (bSimPass == 1 && (PERMUTATION_INPUT_TYPE == INPUT_SAMPLES))
			{
				bValid = (bRestPass==1 && bRest) || (bRestPass==0 && bDeform);
			}
		}

		// Settings
		NormalScale = AddSlider(Ctx, TEXT("Size"), 0.02f, FontWhite) * 10.f; 

		// Position Offsets
		{
			bool bRest   = false;
			bool bDeform = false;

			Move(Ctx, StartPos + OffsetPos*3);
			Print(Ctx, TEXT("Position Offsets"), FontYellow); Newline(Ctx);
			{
				bRest     = AddCheckbox(Ctx, TEXT("Offset Rest    "), false, FontWhite);  Print(Ctx, TEXT("||"), FontCyan); Newline(Ctx);
				bDeform   = AddCheckbox(Ctx, TEXT("Offset Deformed"), false, FontWhite);  Print(Ctx, TEXT("||"), FontMagenta); Newline(Ctx);
				Newline(Ctx);	
			}

			if (DispatchThreadId.x == 0)
			{
				if (bRest)
				{
					const float3 PositionOffset = mul(RestPositionOffset[0], RootLocalToWorld).xyz;
					AddCrossWS(PositionOffset, NormalScale, ColorCyan);
				}

				if (bDeform)
				{
					const float3 PositionOffset = mul(DeformedPositionOffset[0], RootLocalToWorld).xyz;
					AddCrossWS(PositionOffset, NormalScale, ColorMagenta);
				}
			}
		}

		if (!bValid)
		{
			return;
		}
	}

#if PERMUTATION_INPUT_TYPE == INPUT_ROOTS  
	const uint RootIndex = DispatchThreadId.x;
	if (RootIndex < MaxRootCount)
	{
		const float3 B  = UnpackBarycentrics(RootBarycentricBuffer[RootIndex]);
		const uint TriangleIndex = RootToUniqueTriangleIndexBuffer[RootIndex];

		FHairMeshBasis Basis;
		FHairMeshTriangle Tri;
		if (DeformedFrameEnable > 0)
		{		
			Tri   = GetHairMeshTriangle(TriangleIndex, DeformedPositionBuffer);
			Basis = GetHairMeshBasis(TriangleIndex, DeformedPositionBuffer, B);
		}
		else
		{
			Tri   = GetHairMeshTriangle(TriangleIndex, RestPositionBuffer);
			Basis = GetHairMeshBasis(TriangleIndex, RestPositionBuffer, B);
		}

		// Triangle positions
		const float3 WP0 = mul(float4(Tri.P0, 1), RootLocalToWorld).xyz;
		const float3 WP1 = mul(float4(Tri.P1, 1), RootLocalToWorld).xyz;
		const float3 WP2 = mul(float4(Tri.P2, 1), RootLocalToWorld).xyz;

		// Triangle normals
		const float3 WN0 = mul(float4(Tri.N0, 0), RootLocalToWorld).xyz;
		const float3 WN1 = mul(float4(Tri.N1, 0), RootLocalToWorld).xyz;
		const float3 WN2 = mul(float4(Tri.N2, 0), RootLocalToWorld).xyz;

		// Basis
		const float3 WO = mul(float4(Basis.BaseO, 1), RootLocalToWorld).xyz;
		const float3 WT = mul(float4(Basis.BaseT, 0), RootLocalToWorld).xyz;
		const float3 WB = mul(float4(Basis.BaseB, 0), RootLocalToWorld).xyz;
		const float3 WN = mul(float4(Basis.BaseN, 0), RootLocalToWorld).xyz;

		// Face normal
		float3 WFaceN;
		{
			const float3 E0 = (Tri.P1 - Tri.P0).xyz;
			const float3 E1 = (Tri.P2 - Tri.P0).xyz;
			const float3 FaceN = normalize(cross(E1, E0));
			WFaceN = mul(float4(FaceN, 0), RootLocalToWorld).xyz;
		}

		// Selection
		const float4 MinMaxUV = GetScreenMinMax(WP0, WP1, WP2);
		const bool bIsInRect = IsInRect(MinMaxUV, CursorUV);
		if (bIsInRect)
		{
			// Selected triangles
			{
				AddLineWS(WP0, WP1, ColorYellow);
				AddLineWS(WP1, WP2, ColorYellow);
				AddLineWS(WP2, WP0, ColorYellow);
			}

			// Vertex normal
			{
				AddLineWS(WP1, WP1 + WN1 * NormalScale, ColorBlue);
				AddLineWS(WP2, WP2 + WN2 * NormalScale, ColorBlue);
				AddLineWS(WP0, WP0 + WN0 * NormalScale, ColorBlue);
			}

			// Triangle normal
			if (0)
			{
				AddLineWS(WP0, WP0 + WFaceN * NormalScale, ColorGreen);
				AddLineWS(WP1, WP1 + WFaceN * NormalScale, ColorGreen);
				AddLineWS(WP2, WP2 + WFaceN * NormalScale, ColorGreen);
			}
		
			// Root basis
			{
				AddLineWS(WO, WO + WT * NormalScale * 0.3f, ColorRed);
				AddLineWS(WO, WO + WB * NormalScale * 0.3f, ColorGreen);
				AddLineWS(WO, WO + WN * NormalScale * 0.3f, ColorBlue);
			}
		}
		else
		{
			if (bDrawTriangle)
			{
				AddLineWS(WP0, WP1, ColorWhite);
				AddLineWS(WP1, WP2, ColorWhite);
				AddLineWS(WP2, WP0, ColorWhite);
			}

			// Vertex normal
			if (bDrawNormal)
			{
				AddLineWS(WP1, WP1 + WN1 * NormalScale, ColorBlue);
				AddLineWS(WP2, WP2 + WN2 * NormalScale, ColorBlue);
				AddLineWS(WP0, WP0 + WN0 * NormalScale, ColorBlue);
			}

			// Triangle normal
			if (bDrawNormal && 0)
			{
				AddLineWS(WP0, WP0 + WFaceN * NormalScale, ColorGreen);
				AddLineWS(WP1, WP1 + WFaceN * NormalScale, ColorGreen);
				AddLineWS(WP2, WP2 + WFaceN * NormalScale, ColorGreen);
			}
		
			// Root basis
			if (bDrawRoot)
			{
				AddLineWS(WO, WO + WN * NormalScale * 0.3f, ColorYellow);
			}
		}
	}

#endif // INPUT_ROOTS

#if PERMUTATION_INPUT_TYPE == INPUT_SAMPLES
	const uint PointIndex = DispatchThreadId.x;	
	if (PointIndex < MaxRootCount)
	{
		float3 P = (DeformedFrameEnable > 0) ? DeformedSamplePositionsBuffer[PointIndex].xyz : RestSamplePositionsBuffer[PointIndex].xyz;
		P = mul(float4(P, 1), RootLocalToWorld).xyz;
		
		const float4 RBFColor = bRestPass==1 ? ColorPurple : ColorYellow;
		AddCrossWS(P, 1.f, RBFColor);
	}
#endif // INPUT_SAMPLES
}
#endif // SHADER_MESH_PROJECTION_HAIR

/////////////////////////////////////////////////////////////////////////////////////////
// Visualized stats for PPLL primary visibility

#if SHADER_PPLL_DEBUG

#include "../ShaderPrint.ush"
#include "HairStrandsVisibilityCommon.ush" 

float PPLLMeanListElementCountPerPixel;
float PPLLMaxTotalListElementCount;

Texture2D<uint> PPLLCounter;
Texture2D<uint> PPLLNodeIndex;
StructuredBuffer<FPackedHairVisPPLL> PPLLNodeData;

RWTexture2D<float4> SceneColorTextureUAV;

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

[numthreads(8, 8, 1)]
void VisibilityDebugPPLLCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	int2 PixelPos = DispatchThreadId.xy;
	float PPLLListElementcount = PPLLCounter[uint2(0, 0)];

	uint NextNodeIndex = PPLLNodeIndex[PixelPos];
	if (NextNodeIndex != 0xFFFFFFFF)
	{
		uint StrandCount = 0;
		do
		{
			StrandCount++;
			FPackedHairVisPPLL Node = PPLLNodeData[NextNodeIndex];
			NextNodeIndex = Node.NextNodeIndex;
		} while (NextNodeIndex != 0xFFFFFFFF);

		float l = saturate(StrandCount / PPLLMeanListElementCountPerPixel);

		SceneColorTextureUAV[PixelPos] = float4(l, 1.0f - l, 0.0f, 0.0f);
	}

	const float2 BarOrigin = 100.0f;
	const float2 BarSize = float2(200.0f, 20.0f);
	if (all(PixelPos >= BarOrigin) && all(PixelPos <= (BarOrigin + BarSize)))
	{
		float2 LocalPos = PixelPos - BarOrigin;
		float MemoryUsage = PPLLListElementcount / PPLLMaxTotalListElementCount;

		if (MemoryUsage > LocalPos.x / BarSize.x)
		{
			SceneColorTextureUAV[PixelPos] = float4(MemoryUsage, 1.0f - MemoryUsage, 1.0f - MemoryUsage, 0.0f);
		}
		else
		{
			SceneColorTextureUAV[PixelPos] = float4(0.2f, 0.2f, 0.2f, 0.0f);
		}

		if (any(PixelPos == BarOrigin) || any(PixelPos == (BarOrigin + BarSize)))
		{
			SceneColorTextureUAV[PixelPos] = float4(1.0f, 1.0f, 1.0f, 0.0f);
		}
	}

	if (DispatchThreadId.x == 0 && DispatchThreadId.y == 0)
	{
		const float PPLLNodeSizeInMb = (4 + 4 + 4) / 1000000.f;
		const float NodePercentage = float(PPLLListElementcount) / float(PPLLMaxTotalListElementCount);

		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(BarOrigin + BarSize*uint2(0, 2)));
		Print(Context, TEXT("Mean Node Per Pixel : "));
		Print(Context, PPLLMeanListElementCountPerPixel, FontEmerald);
		Newline(Context);
		
		Print(Context, TEXT("Nodes Used (%)      : "));
		Print(Context, NodePercentage * 100.f, GetOccupancyColor(NodePercentage));
		Newline(Context);

		Print(Context, TEXT("Nodes Used          : "));
		Print(Context, PPLLListElementcount, FontSilver);
		Newline(Context);

		Print(Context, TEXT("Allocated Nodes (Mb): "));
		Print(Context, PPLLMaxTotalListElementCount * PPLLNodeSizeInMb, FontSilver);
		Newline(Context);

		Print(Context, TEXT("Total               : "));
		Print(Context, PPLLMaxTotalListElementCount, FontOrange);
		Newline(Context);
	}
}

#endif // SHADER_PPLL_DEBUG

//////////////////////////////////////////////////////////////////////////////////////////
// Visualized Deep Opacity Map stats and buffer

#if SHADER_VISUALIZEDOM

#include "../PostProcessCommon.ush"

float DomScale;
float2 OutputResolution;
float2 InvOutputResolution;

int4 HairViewRect;

Texture2D<float> DeepShadowDepthTexture;
Texture2D<float4> DeepShadowLayerTexture;

SamplerState LinearSampler;

bool IsInsideRect(float4 Rect, float2 PixelPos, inout float2 LocalUV)
{
	LocalUV = (PixelPos - Rect.xy) / Rect.zw;

	return 	PixelPos.x > Rect.x && PixelPos.x < Rect.x + Rect.z &&
			PixelPos.y > Rect.y && PixelPos.y < Rect.y + Rect.w;
}

bool IsOnBorder(float2 LocalUV, float4 Rect)
{
	float2 PixelPos = LocalUV * Rect.zw;
	float BorderThickness = 1;

	return
		PixelPos.x < BorderThickness || PixelPos.x > Rect.z - 1 - BorderThickness ||
		PixelPos.y < BorderThickness || PixelPos.y > Rect.w - 1 - BorderThickness;
}

float3 AddBorder(float2 LocalUV, float4 Rect)
{
	const bool bIsOnBorder = IsOnBorder(LocalUV, Rect);
	return bIsOnBorder ? float3(1,1,0) : float3(0,0,0);
}

// Display the DOM textures into some insert rect
#if PERMUTATION_OUTPUT_TYPE == 0
void VisualizeDomPS(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	const float TileSize = 48;
	const float2 UV = Input.UV;
	const float2 PixelPos = UV * OutputResolution;

	float2 PixelPosAtTileCenter = PixelPos - (frac(PixelPos / TileSize) - 0.5f) * TileSize;
	float2 UVAtTileCenter = PixelPosAtTileCenter * InvOutputResolution;

	const float OffsetX = 450;
	const float OffsetY = 50;
	const float RectSize = 300;
	const float4 DepthRect = float4(OffsetX, OffsetY, RectSize, RectSize);
	const float4 DomRect = float4(OffsetX + RectSize, OffsetY, RectSize, RectSize);

	float3 Color = 0;

	float2 LocalUV = 0;
	if (IsInsideRect(DepthRect, PixelPos, LocalUV))
	{
		const float2 AtlasUV = LocalUV;
		Color = DeepShadowDepthTexture.SampleLevel(LinearSampler, AtlasUV, 0).xxx;
		Color += AddBorder(LocalUV, DepthRect);
	}
	else if (IsInsideRect(DomRect, PixelPos, LocalUV))
	{
		const float2 AtlasUV = LocalUV;
		Color = DeepShadowLayerTexture.SampleLevel(LinearSampler, AtlasUV, 0).xyz / DomScale;
		Color += AddBorder(LocalUV, DomRect);
#if 0 
		float4 Weight = PostprocessInput2.SampleLevel(LinearSampler, LocalUV, 0);
		if (Weight.w > 0) Color = float3(0, 0, 1);
		if (Weight.z > 0) Color = float3(0, 1, 0);
		if (Weight.y > 0) Color = float3(1, 1, 0);
		if (Weight.x > 0) Color = float3(1, 0, 0);
		Color *= 1000;
#endif
	}
	else
	{
		discard;
	}

	OutColor = float4(Color, 1);
}
#endif // PERMUTATION_OUTPUT_TYPE == 0

// Display screen space rect for vizualizing projected viewrect of each hair macro group
#if PERMUTATION_OUTPUT_TYPE == 1
void VisualizeDomPS(
	in FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	const float2 UV = Input.UV;
	const float2 PixelPos = UV * OutputResolution;

	float3 Color = 0;
	float2 LocalUV = 0;
	if (IsInsideRect(HairViewRect, PixelPos, LocalUV) && IsOnBorder(LocalUV, HairViewRect))
	{
		Color += float3(0, 1, 0);
	}
	else
	{
		discard;
	}

	OutColor = float4(Color, 1);
}
#endif // PERMUTATION_OUTPUT_TYPE == 1

#endif // SHADER_VISUALIZEDOM


//////////////////////////////////////////////////////////////////////////////////////////
// Print Deep Opacity Map stats (to be merge with prev. pass)

#if SHADER_DOMINFO

#include "../ShaderPrint.ush"
#include "../Matrices.ush"
#include "../ShadingCommon.ush"
#include "HairStrandsDeepShadowCommonStruct.ush"
#include "HairStrandsAABBCommon.ush"

uint AllocatedSlotCount;
uint MacroGroupCount;
uint bViewRectOptimizeEnabled;
uint bVoxelizationEnabled;
int2 AtlasResolution;

Buffer<int>	MacroGroupAABBBuffer;
StructuredBuffer<FDeepShadowViewInfo> ShadowViewInfoBuffer;

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x > 0)
	{
		return;
	}


	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 40));

	FFontColor FontColor = FontSilver;
	Print(Context, TEXT("Hair Components "), FontYellow); Newline(Context);
	Print(Context, TEXT("R                     : "), FontColor); PrintBool(Context, View.HairComponents & HAIR_COMPONENT_R); Newline(Context);
	Print(Context, TEXT("TT                    : "), FontColor); PrintBool(Context, View.HairComponents & HAIR_COMPONENT_TT); Newline(Context);
	Print(Context, TEXT("TRT                   : "), FontColor); PrintBool(Context, View.HairComponents & HAIR_COMPONENT_TRT); Newline(Context);
	Print(Context, TEXT("Global Scatt.         : "), FontColor); PrintBool(Context, View.HairComponents & HAIR_COMPONENT_GS); Newline(Context);
	Print(Context, TEXT("Local Scatt.          : "), FontColor); PrintBool(Context, View.HairComponents & HAIR_COMPONENT_LS); Newline(Context);
	Newline(Context);

	Print(Context, TEXT("Misc "), FontYellow); Newline(Context); 
	Print(Context, TEXT("Voxelization          : "), FontColor); PrintBool(Context, bVoxelizationEnabled); Newline(Context);
	Print(Context, TEXT("View rect optim.      : "), FontColor); PrintBool(Context, bViewRectOptimizeEnabled); Newline(Context);
	Newline(Context);

	Print(Context, TEXT("DOM "), FontYellow); Newline(Context);
	Print(Context, TEXT("Atlas resolution      : "), FontColor); Print(Context, AtlasResolution.x, FontColor, 4, 0); Print(Context, TEXT(" x "), FontColor); Print(Context, AtlasResolution.y, FontColor, 4, 0); Newline(Context);
	Print(Context, TEXT("Atlas slots           : "), FontColor); Print(Context, AllocatedSlotCount, FontColor); Newline(Context);
	Print(Context, TEXT("GPU driven            : "), FontColor); PrintBool(Context, true); Newline(Context);
	Newline(Context);

	Print(Context, TEXT("Bounds "), FontYellow); Newline(Context);
	const bool bMacroAABB  = AddCheckbox(Context, TEXT("Macro  AABB "), false, FontColor); Newline(Context);
	const bool bShadowAABB = AddCheckbox(Context, TEXT("Shadow AABB "), false, FontColor); Newline(Context);

	if (bMacroAABB)
	{
		for (uint MacroGroupIt = 0; MacroGroupIt < MacroGroupCount; ++MacroGroupIt)
		{
			const FHairAABB Bound = ReadHairAABB(MacroGroupIt, MacroGroupAABBBuffer);
			AddAABBTWS(Bound.Min, Bound.Max, GetGPUAABBMacroGroupBoundVoxelAlignedColor());
		}
	}

	if (bShadowAABB)
	{
		const float4 FrustumColor = float4(0.5f, 1, 0, 1);
		LOOP
		for (uint SlotIt = 0; SlotIt < AllocatedSlotCount; ++SlotIt)
		{
			const float4x4 TranslatedWorldToClip = ShadowViewInfoBuffer[SlotIt].TranslatedWorldToClip;
			const float4x4 ClipToTranslatedWorld = Inverse(TranslatedWorldToClip);

			float4 P01 = float4(-1, -1, 0, 1);
			float4 P11 = float4(-1,  1, 0, 1);
			float4 P21 = float4( 1,  1, 0, 1);
			float4 P31 = float4( 1, -1, 0, 1);

			float4 P00 = float4(-1, -1, 1, 1);
			float4 P10 = float4(-1,  1, 1, 1);
			float4 P20 = float4( 1,  1, 1, 1);
			float4 P30 = float4( 1, -1, 1, 1);

			P00 = mul(P00, ClipToTranslatedWorld); P00 /= P00.w;
			P10 = mul(P10, ClipToTranslatedWorld); P10 /= P10.w;
			P20 = mul(P20, ClipToTranslatedWorld); P20 /= P20.w;
			P30 = mul(P30, ClipToTranslatedWorld); P30 /= P30.w;

			P01 = mul(P01, ClipToTranslatedWorld); P01 /= P01.w;
			P11 = mul(P11, ClipToTranslatedWorld); P11 /= P11.w;
			P21 = mul(P21, ClipToTranslatedWorld); P21 /= P21.w;
			P31 = mul(P31, ClipToTranslatedWorld); P31 /= P31.w;

			AddLineTWS(P00.xyz, P10.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P10.xyz, P20.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P20.xyz, P30.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P30.xyz, P00.xyz, FrustumColor, FrustumColor);

			AddLineTWS(P00.xyz, P01.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P10.xyz, P11.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P20.xyz, P21.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P30.xyz, P31.xyz, FrustumColor, FrustumColor);

			AddLineTWS(P01.xyz, P11.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P11.xyz, P21.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P21.xyz, P31.xyz, FrustumColor, FrustumColor);
			AddLineTWS(P31.xyz, P01.xyz, FrustumColor, FrustumColor);
		}
	}
}
#endif // SHADER_DOMINFO

////////////////////////////////////////////////////////////////////////////////////////////////


////////////////////////////////////////////////////////////////////////////////////////////////
// Print hair strands stats & material infos

#if SHADER_PRINT

#include "../SceneData.ush"
#include "../ShaderPrint.ush"
#include "../CommonViewUniformBuffer.ush"
#include "../SceneTextureParameters.ush"
#include "../DeferredShadingCommon.ush"
#include "../ShaderPrint.ush"
#include "../PositionReconstructionCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsAABBCommon.ush" 

float ResolutionScale;
int2 MaxResolution;
uint FastResolveMask;
uint HairMacroGroupCount;
uint2 GroupSize; 
uint HairVisibilityNodeGroupSize;
uint AllocatedSampleCount;
uint HairInstanceCount;

Buffer<uint> HairInstanceDataBuffer;
Buffer<int> InstanceAABBBuffer;

Texture2D<float> HairCountTexture;
Texture2D<uint> HairCountUintTexture;

Texture2D<uint2> StencilTexture;
SamplerState LinearSampler;
Buffer<uint> HairVisibilityIndirectArgsBuffer;

Buffer<int> HairMacroGroupAABBBuffer;
Buffer<int> HairMacroGroupVoxelAlignedAABBBuffer;

#define MATERIAL_DATA 1
#define DETAIL_DATA 1

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

void DrawInstanceAABB(uint PrimitiveId)
{
	// Since we don't have the actual instance ID (they are transient instance vs. persistent instance), 
	// we find the primitive with the matching PrimitiveID
	uint InstanceID = ~0;
	const uint TotalInstanceCount = min(8192u, uint(GetSceneData().MaxAllocatedInstanceId));
	for (uint LocalInstanceID = 0; LocalInstanceID < TotalInstanceCount; ++LocalInstanceID)
	{
		FInstanceSceneData InstanceData = GetInstanceSceneData(LocalInstanceID);
		if (InstanceData.PrimitiveId == PrimitiveId)
		{
			InstanceID = LocalInstanceID;
			break;
		}
	}

	if (InstanceID != ~0)
	{
		FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceID);
		AddOBBWS(InstanceData.LocalBoundsCenter - InstanceData.LocalBoundsExtent, InstanceData.LocalBoundsCenter + InstanceData.LocalBoundsExtent, GetGPUSceneBoundColor(), DFHackToFloat(InstanceData.LocalToWorld));
		AddReferentialWS(DFHackToFloat(InstanceData.LocalToWorld), 50.f);
	}
}

struct FData
{
	uint InstanceID;
	uint RegisteredIndex;
	uint GeometryType;
	float4 InstanceScreenSphereBound;
};

FData LoadInstanceData(uint InIndex)
{
	const uint Index8 = InIndex*8;
	FData Out;
	Out.InstanceID					= HairInstanceDataBuffer[Index8+0];
	Out.RegisteredIndex				= HairInstanceDataBuffer[Index8+1];
	Out.GeometryType				= HairInstanceDataBuffer[Index8+2];

	Out.InstanceScreenSphereBound.x	= asfloat(HairInstanceDataBuffer[Index8+4]);
	Out.InstanceScreenSphereBound.y	= asfloat(HairInstanceDataBuffer[Index8+5]);
	Out.InstanceScreenSphereBound.z	= asfloat(HairInstanceDataBuffer[Index8+6]);
	Out.InstanceScreenSphereBound.w	= asfloat(HairInstanceDataBuffer[Index8+7]);
	return Out;
}
[numthreads(1, 1, 1)]
void MainCS(uint InGroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{	
	// Draw AABB of the instance groups
	if (all(DispatchThreadId == 0))
	{
		// Draw bound & legends
		{
			FShaderPrintContext CtxL = InitShaderPrintContext(true, uint2(900, 100));
			bool bDrawGPUScene 					= false;
			bool bDrawInstanceGroupScreen		= false;
			bool bDrawHairInstances 			= false;
			bool bDrawMacroBounds 				= false;
			bool bDrawVoxelAlignedMacroBounds	= false;

			// Bounds legends
			{
				bDrawGPUScene 				= AddCheckbox(CtxL, TEXT("Bound - GPU Scene"), 							true, FFontColor(GetGPUSceneBoundColor().xyz)); 						Newline(CtxL);
				bDrawInstanceGroupScreen	= AddCheckbox(CtxL, TEXT("Bound - Instance Group Screen"), 				true, FFontColor(GetInstanceGroupScreenBoundColor().xyz)); 				Newline(CtxL);
				bDrawHairInstances 			= AddCheckbox(CtxL, TEXT("Bound - AABB Instance Group"), 				true, FFontColor(GetGPUAABBInstanceGroupBoundColor().xyz)); 			Newline(CtxL);
				bDrawMacroBounds 			= AddCheckbox(CtxL, TEXT("Bound - AABB Macro Group"), 					true, FFontColor(GetGPUAABBMacroGroupBoundColor().xyz)); 				Newline(CtxL);
				bDrawVoxelAlignedMacroBounds= AddCheckbox(CtxL, TEXT("Bound - AABB Macro Group (voxel-aligned)"),	true, FFontColor(GetGPUAABBMacroGroupBoundVoxelAlignedColor().xyz)); 	Newline(CtxL);
			}

			// Macro groups bounds
			if (bDrawMacroBounds)
			{
				for (uint GroupIndex =0;GroupIndex< HairMacroGroupCount; GroupIndex++)
				{
					FHairAABB Bound = ReadHairAABB(GroupIndex, HairMacroGroupAABBBuffer);
					AddAABBTWS(Bound.Min, Bound.Max, GetGPUAABBMacroGroupBoundColor());
				}
			}
	
			// Voxel aligned macro groups bounds
			if (bDrawVoxelAlignedMacroBounds)
			{
				for (uint GroupIndex = 0; GroupIndex < HairMacroGroupCount; GroupIndex++)
				{
					FHairAABB Bound = ReadHairAABB(GroupIndex, HairMacroGroupVoxelAlignedAABBBuffer);
					AddAABBTWS(Bound.Min, Bound.Max, GetGPUAABBMacroGroupBoundVoxelAlignedColor());
				}
			}
	
			// Hair instance
			for (uint HairInstanceIndex = 0; HairInstanceIndex < HairInstanceCount; HairInstanceIndex++)
			{
				const FData InstanceData = LoadInstanceData(HairInstanceIndex);

				// GPU scene bounds
				if (bDrawGPUScene)
				{
					if (InstanceData.InstanceID != ~0)
					{
						DrawInstanceAABB(InstanceData.InstanceID);
					}
				}

				// Instance AABB
				if (bDrawHairInstances && InstanceData.GeometryType == 0/*Strands*/)
				{
					const FHairAABB AABB = ReadHairAABB(InstanceData.RegisteredIndex, InstanceAABBBuffer);
					AddAABBTWS(AABB.Min, AABB.Max, GetGPUAABBInstanceGroupBoundColor());
				}

				// Screen AABB/size
				if (bDrawInstanceGroupScreen && InstanceData.GeometryType == 0/*Strands*/)
				{
					FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));
					AddCircleSS(Context, InstanceData.InstanceScreenSphereBound.xy, InstanceData.InstanceScreenSphereBound.w, ColorGreen);
				}
			}
		}

		const uint DispatchX = HairVisibilityIndirectArgsBuffer[0];
		const uint DispatchY = HairVisibilityIndirectArgsBuffer[1];
		const uint UsedSampleCount = HairStrands.HairSampleCount[0];
		const uint DispatchedSampleCount = DispatchX * DispatchY * HairVisibilityNodeGroupSize;

		const uint2 EquivalentResolution = GetHairSampleResolution(UsedSampleCount);
		const uint2 AllocatedResolution = HairStrands.HairSampleViewportResolution;
	
		// Pixel coord
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(450, 100));
		Print(Context, TEXT("Sample Count (Used/Alloc)  "), FontCyan);
		PrintLargeNumber(Context, UsedSampleCount, FontCyan);
		Print(Context, TEXT(" / "), FontCyan);
		PrintLargeNumber(Context, AllocatedSampleCount, FontCyan);
		Print(Context, TEXT(" "), FontCyan);

		const float AllocationPercentage = float(UsedSampleCount) / float(AllocatedSampleCount);
		PrintPercentage(Context, AllocationPercentage, GetOccupancyColor(AllocationPercentage));

		Newline(Context);

		Print(Context, TEXT("Sample Lighting Resolution "), FontCyan);
		Print(Context, EquivalentResolution.x, FontCyan, 4);
		Print(Context, TEXT("x"), FontSilver);
		Print(Context, EquivalentResolution.y, FontCyan, 4);
		Print(Context, TEXT(" | "), FontSilver);
		Print(Context, AllocatedResolution.x, FontGrey, 4);
		Print(Context, TEXT("x"), FontSilver);
		Print(Context, AllocatedResolution.y, FontGrey, 4);
		Newline(Context);

		Print(Context, TEXT("MacroGroup Count           "), FontYellow);
		Print(Context, HairMacroGroupCount, FontYellow);
		Newline(Context);
	}

	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 500));	
	const int2 PixelCoord = int2(ResolutionScale * GetCursorPos(Context));
	if (any(PixelCoord < 0) || any(PixelCoord >= MaxResolution))
		return;

	const bool bIsValid = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelCoord, 0)).x > 0;
	if (!bIsValid)
		return;

	const float ViewHairCount = HairCountTexture.Load(uint3(PixelCoord, 0));
	const float ViewHairCountUint = HairCountUintTexture.Load(uint3(PixelCoord, 0)) /  1000.f;
	const float HairPixelCoverage = HairStrands.HairCoverageTexture.Load(uint3(PixelCoord, 0));

	uint Total8BitCoverage = 0;
	const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(uint3(PixelCoord, 0)));
	for (uint SampleIt0 = 0; SampleIt0 < NodeDesc.Count; SampleIt0++)
	{
		const uint LocalOffset = NodeDesc.Offset + SampleIt0;
		const FPackedHairSample NodeData = HairStrands.HairSampleData[LocalOffset];
		const FHairSample Sample = UnpackHairSample(NodeData);
		Total8BitCoverage += Sample.Coverage8bit;
	}

	// Pixel coord
	Print(Context, TEXT("Pixel           "), FontTurquoise);
	Print(Context, PixelCoord, FontTurquoise);
	Newline(Context);

	// Max sample
	Print(Context, TEXT("Max Sample      "));
	Print(Context, HairStrands.MaxSamplePerPixelCount);
	Newline(Context);

	// Unique sample
	Print(Context, TEXT("Sample          "), FontOrange);
	Print(Context, NodeDesc.Count, FontOrange);
	Newline(Context);

	// Accurate coverage (if coverage pass is activated)
	Print(Context, TEXT("Hair Count      "));
	Print(Context, ViewHairCount);
	Newline(Context);

	// Accurate coverage (if coverage pass is activated) uint
	Print(Context, TEXT("Hair Count Uint "));
	Print(Context, ViewHairCountUint);
	Newline(Context);
	
	// Accurate coverage (if coverage pass is activated)
	Print(Context, TEXT("Coverage        "), FontOrange);
	Print(Context, HairPixelCoverage, FontOrange);
	Newline(Context);

	// 8 bit Coverage (summed)
	Print(Context, TEXT("Sum cov. 8bits  "));
	Print(Context, Total8BitCoverage);
	Newline(Context);

	// Resolve type
	const uint Stencil = StencilTexture.Load(uint3(PixelCoord,0)).y;
	const bool bIsFastResolve = (Stencil & FastResolveMask) != 0;
	Print(Context, TEXT("Resolve         "));

	if (bIsFastResolve)
	{
		Print(Context, TEXT("Fast"), FontEmerald);
	}
	else
	{
		Print(Context, TEXT("Regular"), FontEmerald);
	}

	Newline(Context);
	Newline(Context);

	const float2 ColumnPos = Context.Pos;

	// Sample Description
	{	
		Print(Context, TEXT("Sample ID       "), FontWhite);	Newline(Context);
	#if MATERIAL_DATA
		Print(Context, TEXT(".Base Color R   "), FontRed);		Newline(Context); 
		Print(Context, TEXT(".Base Color G   "), FontGreen);	Newline(Context);
		Print(Context, TEXT(".Base Color B   "), FontBlue);		Newline(Context);
		Newline(Context);

		Print(Context, TEXT(".Tangent X      "), FontEmerald);	Newline(Context); 
		Print(Context, TEXT(".Tangent Y      "), FontEmerald);	Newline(Context);
		Print(Context, TEXT(".Tangent Z      "), FontEmerald);	Newline(Context);
		Newline(Context);

		Print(Context, TEXT(".Roughness      "), FontOrange);	Newline(Context);
		Print(Context, TEXT(".Specular       "), FontEmerald);	Newline(Context);
		Print(Context, TEXT(".Backlit        "), FontOrange);	Newline(Context);
		Newline(Context);
	#endif 													  
	#if DETAIL_DATA											  
		Print(Context, TEXT(".Light Channels "), FontEmerald);	Newline(Context);
		Print(Context, TEXT(".Scatter Scene  "), FontOrange);	Newline(Context);
		Print(Context, TEXT(".Depth          "), FontEmerald);	Newline(Context);
		Newline(Context);

		Print(Context, TEXT(".Macro Group ID "), FontYellow);	Newline(Context);
		Print(Context, TEXT(".Primitive		 "), FontEmerald);	Newline(Context);
	#endif													  
		Print(Context, TEXT(".Coverage       "), FontOrange);	Newline(Context);
	}

	// Per sample description
	for (uint SampleIt = 0; SampleIt < NodeDesc.Count; SampleIt++)
	{
		const uint LocalOffset = NodeDesc.Offset + SampleIt;
		const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[LocalOffset]);

		FShaderPrintContext Context2 = InitShaderPrintContext(true, ColumnPos + float2(150.f + 80.f * SampleIt, 0.f) / float2(MaxResolution));

		Print(Context2, SampleIt, FontWhite);									Newline(Context2);

	#if MATERIAL_DATA
		Print(Context2, Sample.BaseColor.x,	FontRed);							Newline(Context2);
		Print(Context2, Sample.BaseColor.y,	FontGreen);							Newline(Context2);
		Print(Context2, Sample.BaseColor.z,	FontBlue);							Newline(Context2);
		Newline(Context2);

		Print(Context2, Sample.Tangent.x,	FontEmerald);						Newline(Context2);
		Print(Context2, Sample.Tangent.y,	FontEmerald);						Newline(Context2);
		Print(Context2, Sample.Tangent.z,	FontEmerald);						Newline(Context2);
		Newline(Context2);

		Print(Context2, Sample.Roughness,	FontOrange);						Newline(Context2);
		Print(Context2, Sample.Specular,		FontEmerald);						Newline(Context2);
		Print(Context2, Sample.Backlit,		FontOrange);						Newline(Context2);
		Newline(Context2);

		// Draw tangent vector (strands)
		{
			//const float2 PixelCoord = GetCursorPos();// DispatchThreadId.xy + 0.5f;
			const float2 UV = PixelCoord / float2(MaxResolution);

			const float SceneDepth = ConvertFromDeviceZ(Sample.Depth);
			const float3 WorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, SceneDepth);
			AddLineTWS(WorldPosition, WorldPosition + Sample.Tangent, ColorRed, ColorYellow);
		}
	#endif 

	#if DETAIL_DATA
		Print(Context2,(Sample.LightChannelMask & 0x1) ? 1u : 0u, FontEmerald);	Newline(Context2);
//		Print(Context2,(Sample.LightChannelMask & 0x1) ? 1u : 0u);				Newline(Context2);
//		Print(Context2,(Sample.LightChannelMask & 0x2) ? 1u : 0u);				Newline(Context2);
//		Print(Context2,(Sample.LightChannelMask & 0x4) ? 1u : 0u);				Newline(Context2);
		Print(Context2, HasHairFlags(Sample.Flags, HAIR_FLAGS_SCATTER_SCENE_LIGHT) ? 1u : 0u, FontOrange); Newline(Context2);
		Print(Context2, Sample.Depth, FontEmerald);								Newline(Context2);
		Newline(Context2);

		Print(Context2, Sample.MacroGroupId, FontYellow);						Newline(Context2);
		Print(Context2, Sample.ControlPointId, FontEmerald);						Newline(Context2);
	#endif
		Print(Context2, Sample.Coverage8bit, FontOrange);						Newline(Context2);
	}
}

#endif // SHADER_PRINT

////////////////////////////////////////////////////////////////////////////////////////////////
// Print stats/info about hair instance (strands/cards/meshes)

#if SHADER_PRINT_INSTANCE

#include "HairStrandsAABBCommon.ush"

uint					InstanceRegisteredIndex;
int2					MaxResolution;
uint					InstanceCount;
uint					InstanceCount_StrandsPrimaryView;
uint					InstanceCount_StrandsShadowView;
uint					InstanceCount_CardsOrMeshesPrimaryView;
uint					InstanceCount_CardsOrMeshesShadowView;

Buffer<uint>			Infos;
Buffer<int>				InstanceAABB;
float4					InstanceScreenSphereBound;

FSTRINGS(InstanceNames)
FSTRINGS(AttributeNames)

struct FInstanceInfo
{
	uint  GroupIndex;
	uint  GroupCount;
	uint  LODCount;
	uint  GeometryType;
	uint  BindingType;
	bool  bHasSim;
	bool  bHasRBF;
	bool  bHasRTGeom;
	float LOD;
	float ClipValue;

	// Strands data
	uint StrandsCount;
	uint VertexCount;
	float ScreenSize;
	uint ActivePointCount;
	uint ActiveCurveCount;
	uint Attributes;
	float Radius;
	float Density;
	float TipScale;
	float RootScale;
	float Length;
	float LengthScale;
	float CoverageScale;
	bool Voxel;
	bool CacheRen;
	bool CacheSim;
	bool bHasLODSwitched;
	bool bAutoLOD;
	bool bCurveRemapping;
	bool bPointRemapping;
	uint Flags;

	// Binding
	uint UniqueTriangle;
	uint UniqueSection;
	uint RootCount;
	uint ControlPointCount;

	uint bIsVisibleInPrimaryView;
	uint bIsVisibleInShadowView;
};

FInstanceInfo ReadInstanceInfo(uint Index)
{
	const uint Index20 = Index * 20u;

	uint4 Data0 = 0;
	Data0.x = Infos[Index20 + 0];
	Data0.y = Infos[Index20 + 1];
	Data0.z = Infos[Index20 + 2];
	Data0.w = Infos[Index20 + 3];

	uint4 Data1 = 0;
	Data1.x = Infos[Index20 + 4];
	Data1.y = Infos[Index20 + 5];
	Data1.z = Infos[Index20 + 6];
	Data1.w = Infos[Index20 + 7];

	uint4 Data2 = 0;
	Data2.x = Infos[Index20 + 8];
	Data2.y = Infos[Index20 + 9];
	Data2.z = Infos[Index20 + 10];
	Data2.w = Infos[Index20 + 11];

	uint4 Data3 = 0;
	Data3.x = Infos[Index20 + 12];
	Data3.y = Infos[Index20 + 13];
	Data3.z = Infos[Index20 + 14];
	Data3.w = Infos[Index20 + 15];

	uint4 Data4 = 0;
	Data4.x = Infos[Index20 + 16];
	Data4.y = Infos[Index20 + 17];
	Data4.z = Infos[Index20 + 18];
	Data4.w = Infos[Index20 + 19];

	FInstanceInfo Out = (FInstanceInfo)0;
	Out.GroupIndex 		= (Data0.x >> 0) & 0xFF;
	Out.GroupCount 		= (Data0.x >> 8) & 0xFF;
	Out.LODCount 		= (Data0.x >> 16) & 0xFF;
	Out.GeometryType 	= (Data0.x >> 24) & 0x7;
	Out.BindingType 	= (Data0.x >> 27) & 0x7;
	Out.bHasSim 		= (Data0.x >> 30) & 0x1;
	Out.bHasRBF 		= (Data0.x >> 31) & 0x1;

	Out.LOD 			= f16tof32((Data0.y >> 0) & 0xFFFF);
	Out.ClipValue 		= f16tof32((Data0.y >> 16) & 0xFFFF);

	Out.StrandsCount 	= Data0.z;
	Out.VertexCount 	= Data0.w;

	// Strands binding data
	Out.UniqueSection		= Data1.x;
	Out.UniqueTriangle		= Data1.y;
	Out.RootCount			= Data1.z;
	Out.ControlPointCount	= Data1.w;

	Out.bIsVisibleInPrimaryView = Data2.x & 0x1;
	Out.bIsVisibleInShadowView	= Data2.x & 0x2;
	Out.Voxel					= Data2.x & 0x4;
	Out.CacheSim				= Data2.x & 0x8;
	Out.CacheRen				= Data2.x & 0x10;
	Out.Flags					=(Data2.x>>8) & 0xFF;
	Out.ScreenSize				= f16tof32((Data2.x>>16) & 0xFFFF);
	Out.ActivePointCount		= Data2.y;
	Out.ActiveCurveCount		= Data2.z;
	Out.Attributes				= Data2.w;
	Out.bAutoLOD				= (Data3.w >> 16u) & 0x1;
	Out.bCurveRemapping			= (Data3.w >> 17u) & 0x1;
	Out.bPointRemapping			= (Data3.w >> 18u) & 0x1;

	Out.Radius					= f16tof32(Data3.x & 0xFFFF);
	Out.Density					= f16tof32(Data3.x >> 16u);
	Out.RootScale				= f16tof32(Data3.y & 0xFFFF);
	Out.TipScale				= f16tof32(Data3.y >> 16u);
	Out.Length					= f16tof32(Data3.z & 0xFFFF);
	Out.LengthScale				= f16tof32(Data3.z >> 16u);
	Out.CoverageScale			= f16tof32(Data3.w & 0xFFFF);

	Out.bHasLODSwitched			= Data4.x & 0x1;
	Out.bHasRTGeom				= Data4.x & 0x2;

	return Out;
}

bool HasAttribute(FInstanceInfo In, uint InAttributeIt)
{
	return (In.Attributes & (1u<< InAttributeIt)) != 0;
}

[numthreads(1, 1, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw AABB of the instance groups
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 100));
	bool bShowInstanceDetails = false;
	bool bShowAdvancedStrandsData = false;

	// Draw totals
	const uint TotalGroup = InstanceCount;
	uint TotalStrandsCount = 0;
	uint TotalVertexCount = 0;
	{
		for (uint InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
		{
			const FInstanceInfo Info = ReadInstanceInfo(InstanceIndex);
			TotalStrandsCount += Info.StrandsCount;
			TotalVertexCount += Info.VertexCount;
		}
	}

	{
		Print(Context, TEXT("Total Groups        : "), FontWhite);
		Print(Context,TotalGroup, FontYellow, 8);
		Newline(Context);

		Print(Context, TEXT("Total Strand Curves : "), FontWhite);
		PrintLargeNumber(Context,TotalStrandsCount, FontOrange);
		Newline(Context);

		Print(Context, TEXT("Total Strand Points : "), FontWhite);
		PrintLargeNumber(Context,TotalVertexCount, FontSilver);
		Newline(Context);
	}
	Newline(Context);

	{
		Print(Context, TEXT("Total Strand        : "), FontWhite);
		Print(Context, TEXT(" Visible:"), FontSilver);
		Print(Context,InstanceCount_StrandsPrimaryView, FontYellow, 4);
		Print(Context, TEXT("| Shadow:"), FontSilver);
		Print(Context,InstanceCount_StrandsShadowView, FontYellow, 4);
		Newline(Context);

		Print(Context, TEXT("Total Cards/Meshes  : "), FontWhite);
		Print(Context, TEXT(" Visible:"), FontSilver);
		Print(Context,InstanceCount_CardsOrMeshesPrimaryView, FontYellow, 4);
		Print(Context, TEXT("| Shadow:"), FontSilver);
		Print(Context, InstanceCount_CardsOrMeshesShadowView, FontYellow, 4);
		Newline(Context);

		bShowInstanceDetails     = AddCheckbox(Context, TEXT("Details"), true, FontSilver);
		bShowAdvancedStrandsData = AddCheckbox(Context, TEXT("Advanced"), false, FontSilver);
		Newline(Context);
	}
	Newline(Context);

	if (bShowInstanceDetails)
	{
		// Draw instance details
		Print(Context, TEXT("   Group  LOD     Geometry Binding Cache Sim RBF Clip Screen CovSc. Strands            Points             Bind.Tri/Sect Vis. Shad. SwLOD RT  Name"));
		Newline(Context);

		LOOP
		for (uint InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
		{
			const FInstanceInfo Info = ReadInstanceInfo(InstanceIndex);

			// Index
			Print(Context,InstanceIndex, FontSilver, 2);
			Print(Context, TEXT(" "));

			// Group
			Print(Context,Info.GroupIndex, FontYellow, 1);
			Print(Context, TEXT("/"));
			Print(Context,Info.GroupCount, FontYellow, 1);
			Print(Context, TEXT("    "));

			// LOD
			if (Info.bAutoLOD)
			{
				Print(Context, TEXT("Auto    "), FontYellow);
			}
			else
			{
				Print(Context,Info.LOD, FontYellow, 4, 2);
				Print(Context, TEXT("/"));
				Print(Context,Info.LODCount-1, FontYellow, 1);
				Print(Context, TEXT("  "));
			}

			// Geometry
			if (Info.GeometryType == 0)		{ Print(Context, TEXT("Strands "), FontEmerald); }
			else if (Info.GeometryType == 1){ Print(Context, TEXT("Cards   "), FontEmerald); }
			else if (Info.GeometryType == 2){ Print(Context, TEXT("Meshes  "), FontEmerald); }
			Print(Context, TEXT(" "));

			// Binding
			if (Info.BindingType == 0)		{ Print(Context, TEXT("None    "), FontTurquoise); }
			else if (Info.BindingType == 1)	{ Print(Context, TEXT("Rigid   "), FontTurquoise); }
			else if (Info.BindingType == 2)	{ Print(Context, TEXT("Skinned "), FontTurquoise); }

			// Cache
			if (Info.CacheRen)      { Print(Context, TEXT("Ren  "), FontOrange); }
			else if (Info.CacheSim) { Print(Context, TEXT("Sim  "), FontOrange); }
			else                    { Print(Context, TEXT("None "), FontSilver); }
			Print(Context, TEXT(" "));
			// Sim
			PrintBool(Context, Info.bHasSim);

			// RBF
			PrintBool(Context, Info.bHasRBF);

			// Clip
			if (Info.ClipValue >= 0)
				Print(Context,Info.ClipValue, FontGreen, 5,3);
			else 
				Print(Context,0u, FontSilver, 5, 3);

			// Strands effective
			Print(Context, Info.ScreenSize, FontOrange, 4, 2);
			Print(Context, TEXT("   "));

			Print(Context, Info.CoverageScale, FontOrange, 4, 2);
			Print(Context, TEXT("   "));

			// Curve sCount
			const float CurveRatio = Info.ActiveCurveCount / float(Info.StrandsCount);
			PrintPercentage(Context, CurveRatio, InitFontColor(ColorMapTurbo(CurveRatio)));
			PrintLargeNumber(Context, Info.ActiveCurveCount, FontOrange);
			Print(Context, TEXT("/"));
			PrintLargeNumber(Context, Info.StrandsCount, FontSilver);
			Print(Context, TEXT(" "), FontWhite);

			// Point sCount
			const float VertexRatio = Info.ActivePointCount / float(Info.VertexCount);
			PrintPercentage(Context, VertexRatio, InitFontColor(ColorMapTurbo(VertexRatio)));
			PrintLargeNumber(Context, Info.ActivePointCount, FontOrange);
			Print(Context, TEXT("/"));
			PrintLargeNumber(Context, Info.VertexCount, FontSilver);
			Print(Context, TEXT(" "), FontWhite);

			// Binding data
			Print(Context,Info.UniqueTriangle, FontOrange, 8);
			Print(Context, TEXT(" "));
			Print(Context,Info.UniqueSection, FontSilver, 3);
			Print(Context, TEXT("  "));

			// Primary view
			PrintBool(Context, Info.bIsVisibleInPrimaryView);
			Print(Context, TEXT(" "));

			// Shadow view
			PrintBool(Context, Info.bIsVisibleInShadowView);
			Print(Context, TEXT("  "));

			// Has LOD switched
			PrintBool(Context, Info.bHasLODSwitched);
			Print(Context, TEXT("  "));

			// Has RT geom
			PrintBool(Context, Info.bHasRTGeom);
			Print(Context, TEXT("  "));
		
		
			// Name
			PrintInstanceNames(Context, InstanceIndex, FontWhite);

			Newline(Context);
		}
		Newline(Context);
	}

	// Optional advanced debugging
	if (bShowAdvancedStrandsData)
	{
		// Attribute
		Print(Context, TEXT("   Group  "), FontWhite);
		Print(Context, TEXT("Voxel "), FontWhite);
		Print(Context, TEXT("RT  "), FontWhite);
		Print(Context, TEXT("Scatter "), FontWhite);
		Print(Context, TEXT("Radius "), FontWhite);
		Print(Context, TEXT("Root  "), FontWhite);
		Print(Context, TEXT("Tip   "), FontWhite);
		Print(Context, TEXT("CvId PtId "), FontWhite);
		const uint AttributeNameInfoCount = NumAttributeNames();
		LOOP
		for (uint AttributeIt=0; AttributeIt < AttributeNameInfoCount; ++AttributeIt)
		{
			PrintAttributeNames(Context, AttributeIt, FontWhite);
			Print(Context, TEXT(" "));
		}
		Newline(Context);

		LOOP
		for (uint StrandInstanceIndex = 0; StrandInstanceIndex < InstanceCount; ++StrandInstanceIndex)
		{
			const FInstanceInfo Info = ReadInstanceInfo(StrandInstanceIndex);
			if (Info.GeometryType > 0)
			{
				continue;
			}

			// Index
			Print(Context, StrandInstanceIndex, FontSilver, 2);
			Print(Context, TEXT(" "));

			// Group
			Print(Context, Info.GroupIndex, FontYellow, 1);
			Print(Context, TEXT("/"));
			Print(Context, Info.GroupCount, FontYellow, 1);
			Print(Context, TEXT("    "));

			PrintBool(Context, Info.Voxel); Print(Context, TEXT("  "));
			PrintBool(Context, HasHairFlags(Info.Flags, HAIR_FLAGS_RAYTRACING_GEOMETRY)); 
			PrintBool(Context, HasHairFlags(Info.Flags, HAIR_FLAGS_SCATTER_SCENE_LIGHT)); Print(Context, TEXT("    "));
			Print(Context, Info.Radius, FontYellow, 5, 3); Print(Context, TEXT("  "));
			Print(Context, Info.RootScale, FontYellow, 5, 3); Print(Context, TEXT(" "));
			Print(Context, Info.TipScale, FontYellow, 5, 3); Print(Context, TEXT(" "));
			PrintBool(Context, Info.bCurveRemapping); Print(Context, TEXT(" "));
			PrintBool(Context, Info.bPointRemapping); Print(Context, TEXT(" "));

			for (uint AttributeIt = 0; AttributeIt < AttributeNameInfoCount; ++AttributeIt)
			{
				PrintBool(Context, HasAttribute(Info, AttributeIt));
				const uint AttributeLen = LenAttributeNames(AttributeIt);
				if (AttributeLen > 3)
				{
					const uint SpaceCount = AttributeLen - 3u;
					for (uint SpaceIt = 0; SpaceIt < SpaceCount; ++SpaceIt)
					{
						Print(Context, TEXT(" "));
					}
				}
			}

			PrintInstanceNames(Context, StrandInstanceIndex, FontWhite);

			Newline(Context);
		}
	}
}

#endif // SHADER_PRINT_INSTANCE

////////////////////////////////////////////////////////////////////////////////////////////////
// Print tangent direction for strands/cards/meshes

#if SHADER_TANGENT

#include "../CommonViewUniformBuffer.ush"
#include "../SceneTextureParameters.ush"
#include "../DeferredShadingCommon.ush"
#include "../ShaderPrint.ush"
#include "../PositionReconstructionCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsAABBCommon.ush" 

uint TileSize;
int2 TileCount;
float2 OutputResolution;
RWTexture2D<float4> OutputTexture;

#define HAIR_TANGENT_INVALID 0
#define HAIR_TANGENT_STRANDS 1
#define HAIR_TANGENT_CARDS   2


struct FTangentDesc
{
	float3 Tangent;
	uint Type;
};

FTangentDesc AddTangentPlot(uint2 PixelCoord, float2 UV)
{
	FTangentDesc Out = (FTangentDesc)0;

	// Strands
	const float HairDeviceZ = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelCoord,0)).x;
	if (HairDeviceZ > 0)
	{
		// Find closest
		float3 Closest_Tangent = 0;
		float  Closest_Depth = 0;

		uint Total8BitCoverage = 0;
		const FNodeDesc NodeDesc = DecodeNodeDesc(HairStrands.HairSampleOffset.Load(uint3(PixelCoord, 0)));
		for (uint SampleIt0 = 0; SampleIt0 < NodeDesc.Count; SampleIt0++)
		{
			const uint LocalOffset = NodeDesc.Offset + SampleIt0;
			const FPackedHairSample NodeData = HairStrands.HairSampleData[LocalOffset];
			const FHairSample Sample = UnpackHairSample(NodeData);

			if (Sample.Depth > Closest_Depth)
			{
				Closest_Depth = Sample.Depth;
				Closest_Tangent = Sample.Tangent;
			}
		}

		// Draw tangent vector (strands)
		if (NodeDesc.Count > 0)
		{
			const float SceneDepth = ConvertFromDeviceZ(Closest_Depth);
			const float3 WorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, SceneDepth);
			AddLineTWS(WorldPosition, WorldPosition + Closest_Tangent, ColorRed, ColorYellow);
			Out.Type = 1;
			Out.Tangent = Closest_Tangent;
		}
	}

	// Cards
	if (Out.Type == 0)
	{
		FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);
		const float SceneDepth = CalcSceneDepth(UV);
		const float3 WorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, SceneDepth);
		if (ScreenSpaceData.GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			const float3 Tangent = ScreenSpaceData.GBuffer.WorldNormal;
			AddLineTWS(WorldPosition, WorldPosition + Tangent, ColorPurple, ColorYellow);
			Out.Type = 2;
			Out.Tangent = Tangent;
		}
	}

	return Out;
}

void AddTangentText(float2 UV, float3 Tangent, uint TangentType)
{
	FShaderPrintContext Context = InitShaderPrintContext(true, UV);

	if (TangentType == HAIR_TANGENT_STRANDS)
		Print(Context, TEXT("Strands"), FontYellow);
	else if (TangentType == HAIR_TANGENT_CARDS)
		Print(Context, TEXT("Cards "), FontYellow);
	else if (TangentType == HAIR_TANGENT_CARDS)
		Print(Context, TEXT("Invalid"), FontRed);

	Newline(Context);

	Print(Context, Tangent.x, FontRed);		Newline(Context);
	Print(Context, Tangent.y, FontGreen);	Newline(Context);
	Print(Context, Tangent.z, FontBlue);	Newline(Context);
}

[numthreads(8, 8, 1)]
void MainCS(int2 DispatchThreadId : SV_DispatchThreadID)
{
	// Cursor plot
	if (all(DispatchThreadId == 0))
	{
		const float2 PixelCoord = GetCursorPos();// DispatchThreadId.xy + 0.5f;
		const float2 UV = PixelCoord / float2(OutputResolution);
		const float2 UVText = (PixelCoord + float2(20, 0)) / float2(OutputResolution);

		FTangentDesc Desc = AddTangentPlot(PixelCoord, UV);
		if (Desc.Type == HAIR_TANGENT_STRANDS)
		{			
			AddTangentText(UVText, Desc.Tangent, HAIR_TANGENT_STRANDS);
		}
		else if (Desc.Type == HAIR_TANGENT_CARDS)
		{
			AddTangentText(UVText, Desc.Tangent, HAIR_TANGENT_CARDS);
		}
	}

	// Grid plot
	if (all(DispatchThreadId.xy < TileCount))
	{
		const float2 PixelCoord = DispatchThreadId.xy * TileSize;
		if (all(PixelCoord < OutputResolution))
		{
			const float2 UV = PixelCoord / float2(OutputResolution);
			AddTangentPlot(PixelCoord, UV);
		}
	}
}
#endif // SHADER_TANGENT

////////////////////////////////////////////////////////////////////////////////////////////////
// Print LOD info/stats

#if SHADER_LOD_INFO

#include "../ShaderPrint.ush"

int2 MaxResolution;
uint GroupIndex;
uint GeometryType;
uint CurveCount;
uint PointCount;
float CoverageScale;
float LOD;
float ScreenSize;
float3 GroupColor;

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Legend
	if (GroupIndex == 0)
	{
		FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(10, 50));
		Print(Ctx, TEXT("  Group Geom.   LOD  Screen Size #Curve      #Point      Scale"), FontSilver); Newline(Ctx);
	}

	// Values
	{
		FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(10, 65 + GroupIndex * 15));

		// Group Color
		Print(Ctx, TEXT("* "), InitFontColor(GroupColor));

		// Group index
		Print(Ctx, GroupIndex, FontEmerald, 3, 0);

		// Geometry Type
		Print(Ctx, TEXT("   "));
		if (GeometryType == 0)
		{
			Print(Ctx, TEXT("Strands "), FontOrange);
		}
		else if (GeometryType == 1)
		{
			Print(Ctx, TEXT("Cards   "), FontOrange);
		}
		else if (GeometryType == 2)
		{
			Print(Ctx, TEXT("Meshes  "), FontOrange);
		}

		Print(Ctx, LOD, FontWhite, 4, 2); Print(Ctx, TEXT(" "));
		Print(Ctx, saturate(ScreenSize), FontWhite);
		Print(Ctx, CurveCount, FontWhite);
		Print(Ctx, PointCount, FontWhite);
		Print(Ctx, CoverageScale, FontWhite);
		Newline(Ctx);
	}
}

#endif // SHADER_LOD_INFO

/////////////////////////////////////////////////////////////////////////////////////////
// Visualize hair cards atlas data
#if SHADER_CARDS_ATLAS

#include "../ShaderPrint.ush"

#define DEBUG_MODE_DEPTH 1
#define DEBUG_MODE_COVERAGE 2
#define DEBUG_MODE_TANGENT 3
#define DEBUG_MODE_ROOTUV 4
#define DEBUG_MODE_U 5
#define DEBUG_MODE_SEED 6

int2 OutputResolution;
int2 AtlasResolution;
int DebugMode;

Texture2D<float4> AtlasTexture;
RWTexture2D<float4> OutputTexture;
SamplerState LinearSampler;

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{	
	const int2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord >= OutputResolution))
		return;

	const int2 Offset = 50;
	const int2 Resolution = 512 * float2(AtlasResolution.x, AtlasResolution.y)/max(AtlasResolution.x, AtlasResolution.y);

	const float2 LocalUV = float2(PixelCoord - Offset) / float2(Resolution);

#if 1
	if (all(LocalUV >= 0) && all(LocalUV <= 1))
	{
		float4 InData = AtlasTexture.SampleLevel(LinearSampler, float2(LocalUV.x,1-LocalUV.y), 0);
		float3 Data = 0;
		if (DebugMode == DEBUG_MODE_DEPTH) 		{ Data = InData.xxx; }
		if (DebugMode == DEBUG_MODE_COVERAGE)	{ Data = InData.xxx; }
		if (DebugMode == DEBUG_MODE_TANGENT)	{ Data = InData.xyz; }
		if (DebugMode == DEBUG_MODE_ROOTUV)		{ Data = float3(InData.xy,0); }
		if (DebugMode == DEBUG_MODE_U)			{ Data = InData.zzz; }
		if (DebugMode == DEBUG_MODE_SEED)		{ Data = InData.www; }
		OutputTexture[PixelCoord] = float4(Data, 1);
	}
#else
	if (all(PixelCoord <= AtlasResolution))
	{
		float3 Data = AtlasTexture.Load(uint3(PixelCoord, 0)).xyz;
		if (DebugMode == DEBUG_MODE_DEPTH) { Data = Data.xxx; }
		if (DebugMode == DEBUG_MODE_COVERAGE) { Data = Data.xxx; }
		if (DebugMode == DEBUG_MODE_TANGENT) {}
		OutputTexture[PixelCoord] = float4(Data, 1);
	}
#endif

#if PRINT_ATLAS_RESOLUTION
	// Draw AABB of the instance groups
	if (DispatchThreadId.x == 0 && DispatchThreadId.y == 0)
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 40));
		Print(Context, TEXT("Atlas Resolution: "))
		Print(Context, AtlasResolution);
		Newline(Context);
	}
#endif
}

#endif // SHADER_CARDS_ATLAS

/////////////////////////////////////////////////////////////////////////////////////////
// Visualize cards hair guides

#if SHADER_CARDS_GUIDE

#include "HairStrandsVertexFactoryCommon.ush"
#include "../ShaderPrint.ush"

uint InstanceRegisteredIndex;
uint CardLODIndex;

uint RenVertexCount;
float3 RenRestOffset;

uint SimVertexCount;
float3 SimRestOffset;

float4x4 LocalToWorld;

ByteAddressBuffer RenRestPosition;
ByteAddressBuffer RenDeformedPosition;
StructuredBuffer<float4> RenDeformedOffset;

ByteAddressBuffer SimRestPosition;
ByteAddressBuffer SimDeformedPosition;
StructuredBuffer<float4> SimDeformedOffset;

int DebugMode;

#define DEBUGMODE_REN_REST 1
#define DEBUGMODE_REN_DEFORMED 2
#define DEBUGMODE_SIM_REST 3
#define DEBUGMODE_SIM_DEFORMED 4

[numthreads(32, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint VertexId = DispatchThreadId.x;
	const uint VertexCount = DebugMode == DEBUGMODE_REN_REST || DebugMode == DEBUGMODE_REN_DEFORMED ? RenVertexCount : SimVertexCount;
	if (VertexId+1 >= VertexCount)
		return;

	FHairControlPoint P0 = (FHairControlPoint)0;
	FHairControlPoint P1 = (FHairControlPoint)0;

	float4 ColorRoot = ColorWhite;
	float4 ColorTip  = ColorWhite;

	if (DebugMode == DEBUGMODE_REN_REST)
	{
		P0       = ReadHairControlPoint(RenRestPosition, VertexId,     RenRestOffset, 1, 1, 1);
		P1       = ReadHairControlPoint(RenRestPosition, VertexId + 1, RenRestOffset, 1, 1, 1);
		ColorRoot= ColorYellow;
		ColorTip = ColorPurple;
	}
	else if (DebugMode == DEBUGMODE_REN_DEFORMED)
	{
		const float3 Offset = ReadCardPositionOffset(RenDeformedOffset, InstanceRegisteredIndex, CardLODIndex);

		P0       = ReadHairControlPoint(RenDeformedPosition, VertexId,     Offset, 1, 1, 1);
		P1       = ReadHairControlPoint(RenDeformedPosition, VertexId + 1, Offset, 1, 1, 1);
		ColorRoot= ColorYellow;
		ColorTip = ColorPurple;
	}
	else if (DebugMode == DEBUGMODE_SIM_REST)
	{
		P0       = ReadHairControlPoint(SimRestPosition, VertexId,     SimRestOffset, 1, 1, 1);
		P1       = ReadHairControlPoint(SimRestPosition, VertexId + 1, SimRestOffset, 1, 1, 1);
		ColorRoot= ColorCyan;
		ColorTip = ColorRed;
	}
	else if (DebugMode == DEBUGMODE_SIM_DEFORMED)
	{
		const float3 Offset = ReadSimPositionOffset(SimDeformedOffset, InstanceRegisteredIndex);

		P0       = ReadHairControlPoint(SimDeformedPosition, VertexId,     Offset, 1, 1, 1);
		P1       = ReadHairControlPoint(SimDeformedPosition, VertexId + 1, Offset, 1, 1, 1);
		ColorRoot= ColorCyan;
		ColorTip = ColorRed;
	}

	if (P1.Type != HAIR_CONTROLPOINT_START)
	{
		const float4 Color0 = lerp(ColorRoot, ColorTip, P0.UCoord);
		const float4 Color1 = lerp(ColorRoot, ColorTip, P1.UCoord);
		AddLineWS(mul(float4(P0.Position, 1), LocalToWorld).xyz, mul(float4(P1.Position,1), LocalToWorld).xyz, Color0, Color1);
	}
}

#endif // SHADER_CARDS_GUIDE

///////////////////////////////////////////////////////////////////////////////////////////////////
// Visualize directional occlusion from voxelized hair strands (used for cards generation)

#if SHADER_CARDS_DIRECTION_OCCLUSION

int3 Voxel_Resolution;
float Voxel_VoxelSize;
float3 Voxel_MinBound;
float3 Voxel_MaxBound;
Buffer<uint> Voxel_DensityBuffer;
RWBuffer<uint> OutBuffer;

[numthreads(4, 4, 4)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const int3 VoxelCoord = DispatchThreadId.xyz;

	if (any(VoxelCoord >= Voxel_Resolution))
		return;

	const uint VoxelLinearCoord = CoordToIndex(VoxelCoord, Voxel_Resolution, 0);

	// Dilation
#if 0
	uint OutValue   = 0;
	 int KernelSize = 1;
	uint ValidCount = 0;

	for (int Z = -KernelSize; Z <= KernelSize; ++Z)
	for (int Y = -KernelSize; Y <= KernelSize; ++Y)
	for (int X = -KernelSize; X <= KernelSize; ++X)
	{
		int3 C = VoxelCoord + int3(X, Y, Z);
		if (all(C >= 0) && all(C < Voxel_Resolution))
		{
			const uint LinearC = CoordToIndex(C, Voxel_Resolution, 0);
			OutValue += Voxel_DensityBuffer[LinearC];
			ValidCount++;
		}
	}
	OutValue /= max(1, ValidCount);
#endif

	// Occlusion
#if 1
	const uint CenterDensity = Voxel_DensityBuffer[VoxelLinearCoord];
	if (CenterDensity == 0)
		return;

	int KernelSize = 1;
	uint ValidCount = 0;
	float3 OutDirection = 0;
	for (int Z = -KernelSize; Z <= KernelSize; ++Z)
	for (int Y = -KernelSize; Y <= KernelSize; ++Y)
	for (int X = -KernelSize; X <= KernelSize; ++X)
	{
		const int3 Offset = int3(X, Y, Z);		
		const int3 C = VoxelCoord + Offset;
		if (all(C >= 0) && all(C < Voxel_Resolution) && !all(Offset == int3(0,0,0)))
		{
			const uint LinearC = CoordToIndex(C, Voxel_Resolution, 0);
			const uint Density = Voxel_DensityBuffer[LinearC];
			if (Density == 0)
			{
				OutDirection += normalize(float3(Offset));
				ValidCount++;
			}
		}
	}
	if (ValidCount>0)
	{
		OutDirection = normalize(OutDirection);
	}
	const uint OutValue = PackDirection(OutDirection);
#endif

	// Pass through
#if 0
	const uint OutValue = Voxel_DensityBuffer[VoxelLinearCoord];
#endif

	OutBuffer[VoxelLinearCoord] = OutValue;
}

#endif //SHADER_CARDS_DIRECTION_OCCLUSION

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Display groom culling from shadow perspective 

#if SHADER_SHADOW_CULLING

#include "../ShaderPrint.ush"
#include "../Matrices.ush"

void AddFrustumWS(float4x4 In, float4 InColor, bool bInv)
{
	const float Near = 0.0001f;
	const float Far = 0.9999f;

	float4x4 T = In;
	if (bInv)
	{
		T = Inverse(In);
	}
	float4 P00 = mul(float4(-1, -1, Near, 1), T); P00 /= P00.w;
	float4 P10 = mul(float4( 1, -1, Near, 1), T); P10 /= P10.w;
	float4 P20 = mul(float4( 1,  1, Near, 1), T); P20 /= P20.w;
	float4 P30 = mul(float4(-1,  1, Near, 1), T); P30 /= P30.w;

	float4 P01 = mul(float4(-1, -1, Far, 1),  T); P01 /= P01.w;
	float4 P11 = mul(float4( 1, -1, Far, 1),  T); P11 /= P11.w;
	float4 P21 = mul(float4( 1,  1, Far, 1),  T); P21 /= P21.w;
	float4 P31 = mul(float4(-1,  1, Far, 1),  T); P31 /= P31.w;

	AddLineWS(P00.xyz, P10.xyz, InColor);
	AddLineWS(P10.xyz, P20.xyz, InColor);
	AddLineWS(P20.xyz, P30.xyz, InColor);
	AddLineWS(P30.xyz, P00.xyz, InColor);

	AddLineWS(P01.xyz, P11.xyz, InColor);
	AddLineWS(P11.xyz, P21.xyz, InColor);
	AddLineWS(P21.xyz, P31.xyz, InColor);
	AddLineWS(P31.xyz, P01.xyz, InColor);

	AddLineWS(P00.xyz, P01.xyz, InColor);
	AddLineWS(P10.xyz, P11.xyz, InColor);
	AddLineWS(P20.xyz, P21.xyz, InColor);
	AddLineWS(P30.xyz, P31.xyz, InColor);
}

Buffer<float> InstanceBoundInLightSpace;
Buffer<float> InstanceBoundInWorldSpace;
Buffer<uint>  InstanceIntersection;

uint InstanceCount;

float3 LightCenter;
float3 LightExtent;

float4x4 LightToWorld;

float4x4 ViewWorldToProj;
float4x4 ViewProjToWorld;

[numthreads(1, 1, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw AABB of the instance groups
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

	// View Frustum
	//AddFrustumWS(ViewProjToWorld, ColorBlue, false);
	AddFrustumWS(ViewWorldToProj, ColorBlue, true);

	// Light bound
	AddAABBWS(LightCenter - LightExtent, LightCenter + LightExtent, ColorPurple);

	for (uint InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
	{	
		const uint Index6 = InstanceIndex * 6;
		// In light space
		{
			const bool bIntersect = InstanceIntersection[InstanceIndex] > 0;
			const float3 MinP = float3(InstanceBoundInLightSpace[Index6 + 0], InstanceBoundInLightSpace[Index6 + 1], InstanceBoundInLightSpace[Index6 + 2]);
			const float3 MaxP = float3(InstanceBoundInLightSpace[Index6 + 3], InstanceBoundInLightSpace[Index6 + 4], InstanceBoundInLightSpace[Index6 + 5]);
			// Transform back from light space to world space
			AddOBBWS(MinP, MaxP, bIntersect ? ColorGreen : ColorYellow, LightToWorld);
		}
	
		// In world space
		{
			const float3 MinP = float3(InstanceBoundInWorldSpace[Index6 + 0], InstanceBoundInWorldSpace[Index6 + 1], InstanceBoundInWorldSpace[Index6 + 2]);
			const float3 MaxP = float3(InstanceBoundInWorldSpace[Index6 + 3], InstanceBoundInWorldSpace[Index6 + 4], InstanceBoundInWorldSpace[Index6 + 5]);
			AddAABBWS(MinP, MaxP, ColorRed);
		}
	}
}
#endif // SHADER_SHADOW_CULLING

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Display hair picking attribute

#if SHADER_ATTRIBUTE_DEBUG

#include "../Common.ush"
#include "../ShaderPrint.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
HAIR_STRANDS_INSTANCE_PARAMETERS(HairInstance)

#include "HairStrandsAttributeCommon.ush"
#define HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name) HairInstance_##Name
#include "/Engine/Private/HairStrands/HairStrandsAttributeTemplate.ush"

uint MaterialId;
Texture2D<uint> NodeIndexTexture;
StructuredBuffer<FPackedHairVis> NodeVisBuffer;

[numthreads(1, 1, 1)]
void CSMain(
	uint DispatchThreadID : SV_DispatchThreadID, 
	uint GroupThreadID : SV_GroupThreadID, 
	uint GroupID : SV_GroupID)
{
	const int2 PixelCoord = ShaderPrintData.CursorCoord;
	if (all(PixelCoord < 0))
	{
		return;
	}

	const FNodeDesc NodeDesc = DecodeNodeDesc(NodeIndexTexture.Load(uint3(PixelCoord, 0)));
	if (NodeDesc.Count == 0)
	{
		return;
	}

	if (NodeDesc.Count > 0)
	{
		// Only show the first sample properties for simplicity
		const uint SampleIt = 0;
		const uint SampleIndex = NodeDesc.Offset + SampleIt;
		const FHairVis VisNode = UnpackHairVis(NodeVisBuffer[SampleIndex]);
		if (VisNode.MaterialId == MaterialId)
		{
			FHairControlPoint Out = ReadHairControlPoint(
				HairInstance_PositionBuffer,
				VisNode.ControlPointId,
				HairInstance_GetHairInstancePositionOffset(),
				HairInstance_Radius,
				HairInstance_RootScale,
				HairInstance_TipScale);

			FFontColor FontValue = FontYellow;
			FFontColor FontName = FontWhite;

			FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(PixelCoord) + uint2(20, 0));
			Print(Ctx, TEXT("Group    : "), FontName); Print(Ctx, GetHairStrandsGroupIndex(), FontValue); Newline(Ctx);
			Print(Ctx, TEXT("Radius   : "), FontName); Print(Ctx, GetHairStrandsDimensions(VisNode.ControlPointId, 0).y, FontValue); Newline(Ctx);
			Print(Ctx, TEXT("Seed     : "), FontName); Print(Ctx, GetHairStrandsSeed(VisNode.ControlPointId), FontValue); Newline(Ctx);
			Print(Ctx, TEXT("CoordU   : "), FontName); Print(Ctx, GetHairStrandsUV(VisNode.ControlPointId, float2(0, 0.5f)).x, FontValue); Newline(Ctx);
			Newline(Ctx);

			if (HasHairStrandsRootUV()) 	{ Print(Ctx, TEXT("RootUV   : "), FontName); Print(Ctx, GetHairStrandsRootUV(VisNode.ControlPointId),  FontValue); Newline(Ctx); }
			if (HasHairStrandsClumpId()) 	{ Print(Ctx, TEXT("Clump ID : "), FontName); Print(Ctx, GetHairStrandsClumpID(VisNode.ControlPointId).x, FontValue); Newline(Ctx); }
			if (HasHairStrandsClumpId3()) 	{ Print(Ctx, TEXT("Clump IDs: "), FontName); Print(Ctx, GetHairStrandsClumpID(VisNode.ControlPointId), FontValue); Newline(Ctx); }
			if (HasHairStrandsColor()) 		{ Print(Ctx, TEXT("Color    : "), FontName); Print(Ctx, GetHairStrandsColor(VisNode.ControlPointId, 0), FontValue); Newline(Ctx); }
			if (HasHairStrandsRoughness()) 	{ Print(Ctx, TEXT("Roughness: "), FontName); Print(Ctx, GetHairStrandsRoughness(VisNode.ControlPointId, 0), FontValue); Newline(Ctx); }
			if (HasHairStrandsAO()) 		{ Print(Ctx, TEXT("AO       : "), FontName); Print(Ctx, GetHairStrandsAO(VisNode.ControlPointId, 0), FontValue); Newline(Ctx); }
			Newline(Ctx);
		}
	}
}

#endif // SHADER_ATTRIBUTE_DEBUG


////////////////////////////////////////////////////////////////////////////////////////////////
// Print memory

#if SHADER_PRINT_MEMORY

FSTRINGS(GroomNames)
FSTRINGS(BindingNames)
FSTRINGS(ComponentNames)

uint ComponentCount;
uint GroomCount;
uint BindingCount;

Buffer<uint> ComponentBuffer;
Buffer<uint> GroomBuffer;
Buffer<uint> BindingBuffer;

struct FInfo
{
	uint4 Data0;
	uint4 Data1;
	uint4 Data2;
	uint4 Data3;
	uint4 Data4;
};

FInfo ReadInfo(uint Index, Buffer<uint> InBuffer)
{
	const uint Index20 = Index * 20u;
	FInfo Out;
	Out.Data0.x = InBuffer[Index20 + 0];
	Out.Data0.y = InBuffer[Index20 + 1];
	Out.Data0.z = InBuffer[Index20 + 2];
	Out.Data0.w = InBuffer[Index20 + 3];
	Out.Data1.x = InBuffer[Index20 + 4];
	Out.Data1.y = InBuffer[Index20 + 5];
	Out.Data1.z = InBuffer[Index20 + 6];
	Out.Data1.w = InBuffer[Index20 + 7];
	Out.Data2.x = InBuffer[Index20 + 8];
	Out.Data2.y = InBuffer[Index20 + 9];
	Out.Data2.z = InBuffer[Index20 + 10];
	Out.Data2.w = InBuffer[Index20 + 11];
	Out.Data3.x = InBuffer[Index20 + 12];
	Out.Data3.y = InBuffer[Index20 + 13];
	Out.Data3.z = InBuffer[Index20 + 14];
	Out.Data3.w = InBuffer[Index20 + 15];
	Out.Data4.x = InBuffer[Index20 + 16];
	Out.Data4.y = InBuffer[Index20 + 17];
	Out.Data4.z = InBuffer[Index20 + 18];
	Out.Data4.w = InBuffer[Index20 + 19];
	return Out;
}

struct FComponent
{
	uint GroupIndex;
	uint GroupCount;
	uint RestCurve;
	uint ActiveCurve;

	uint GPU_Guides;
	uint GPU_Strands;
	uint GPU_Cards;
	uint GPU_Meshes;
};

FComponent GetComponent(uint Index)
{	
	const FInfo In = ReadInfo(Index, ComponentBuffer);

	FComponent Out;
	Out.GroupIndex 			=In.Data0.x;
	Out.GroupCount 			=In.Data0.y;
	Out.ActiveCurve			=In.Data0.z;
	Out.RestCurve			=In.Data0.w;
	
	Out.GPU_Guides 			=In.Data1.x;
	Out.GPU_Strands 		=In.Data1.y;
	Out.GPU_Cards 			=In.Data1.z;
	Out.GPU_Meshes 			=In.Data1.w;

	return Out;
}

struct FGroom
{
	uint GroupIndex;
	uint GroupCount;
	uint RestCurve;
	uint ActiveCurve;

	uint CPU_Guides;
	uint CPU_Strands;
	uint CPU_Cards;
	uint CPU_Meshes;

	uint GPU_Guides;
	uint GPU_Strands;
	uint GPU_Cards;
	uint GPU_Meshes;

	uint GPU_Rest;
	uint GPU_Interpolation;
	uint GPU_Cluster;
	uint GPU_Raytracing;

	uint Curve_Rest;
	uint Curve_Interpolation;
	uint Curve_Cluster;
	uint Curve_Raytracing;
};

FGroom GetGroom(uint Index)
{	
	const FInfo In = ReadInfo(Index, GroomBuffer);

	FGroom Out;
	Out.GroupIndex 			= In.Data0.x;
	Out.GroupCount 			= In.Data0.y;
	Out.ActiveCurve			= In.Data0.z;
	Out.RestCurve			= In.Data0.w;

	Out.CPU_Guides 			= In.Data1.x;
	Out.CPU_Strands 		= In.Data1.y;
	Out.CPU_Cards 			= In.Data1.z;
	Out.CPU_Meshes 			= In.Data1.w;

	Out.GPU_Guides 			= In.Data2.x;
	Out.GPU_Strands 		= In.Data2.y;
	Out.GPU_Cards 			= In.Data2.z;
	Out.GPU_Meshes 			= In.Data2.w;

	Out.GPU_Rest			= In.Data3.x;
	Out.GPU_Interpolation	= In.Data3.y;
	Out.GPU_Cluster			= In.Data3.z;
	Out.GPU_Raytracing		= In.Data3.w;

	Out.Curve_Rest			= In.Data4.x;
	Out.Curve_Interpolation	= In.Data4.x;
	Out.Curve_Cluster     	= In.Data4.x;
	Out.Curve_Raytracing  	= In.Data4.x;

	return Out;
}

struct FBinding
{
	uint GroupIndex;
	uint GroupCount;
	uint RestCurve;
	uint ActiveCurve;

	uint CPU_Guides;
	uint CPU_Strands;
	uint CPU_Cards;
	uint CPU_Meshes;

	uint GPU_Guides;
	uint GPU_Strands;
	uint GPU_Cards;
	uint GPU_Meshes;
};

FBinding GetBinding(uint Index)
{	
	const FInfo In = ReadInfo(Index, BindingBuffer);

	FBinding Out;
	Out.GroupIndex 	= In.Data0.x;
	Out.GroupCount 	= In.Data0.y;
	Out.ActiveCurve	= In.Data0.z;
	Out.RestCurve	= In.Data0.w;

	Out.CPU_Guides 	= In.Data1.x;
	Out.CPU_Strands = In.Data1.y;
	Out.CPU_Cards 	= In.Data1.z;
	Out.CPU_Meshes 	= In.Data1.w;

	Out.GPU_Guides 	= In.Data2.x;
	Out.GPU_Strands = In.Data2.y;
	Out.GPU_Cards 	= In.Data2.z;
	Out.GPU_Meshes 	= In.Data2.w;
	return Out;
}

void PrintGroup(inout FShaderPrintContext Ctx, uint GroupIndex, uint GroupCount)
{
	Print(Ctx, GroupIndex,	FontEmerald, 2, 0);	
	Print(Ctx, TEXT("/"), FontWhite);
	Print(Ctx, GroupCount,	FontEmerald, 2, 0);
}

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

void PrintCurve(inout FShaderPrintContext Ctx, uint ActiveCurve, uint RestCurve)
{
	const float Curve = float(ActiveCurve) / float(RestCurve);
	PrintPercentage(Ctx, Curve, GetOccupancyColor(Curve));
	//PrintLargeNumber(Ctx, Info.ActiveCurve,	FontOrange);	Print(Ctx, TEXT(" "), FontWhite);
	//Print(Ctx, TEXT("/"), FontWhite);
	//PrintLargeNumber(Ctx, Info.RestCurve,	FontOrange);	Print(Ctx, TEXT(" "), FontWhite);
}

[numthreads(1, 1, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw AABB of the instance groups
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	// Widgets
	FShaderPrintContext Ctx2 = InitShaderPrintContext(true, uint2(900, 100));
	Print(Ctx2, TEXT("Controls"), FontYellow); Newline(Ctx2);
	const bool bComponent 	= AddCheckbox(Ctx2, TEXT("Components"), false, FontWhite); 	Newline(Ctx2);	
	const bool bGroom     	= AddCheckbox(Ctx2, TEXT("Grooms"), true, FontWhite);     	Newline(Ctx2);	
	const bool bBinding   	= AddCheckbox(Ctx2, TEXT("Bindings"), false, FontWhite);   	Newline(Ctx2);
	const bool bStrandsOnly = AddCheckbox(Ctx2, TEXT("Strands Only"), true, FontWhite); Newline(Ctx2);

	// Component
	FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(50, 100));
	uint Component_TotalGPU = 0;
	if (bComponent)
	{
		Print(Ctx, TEXT("Components"), FontYellow); Newline(Ctx);
		{
			Print(Ctx, TEXT("Grp. Common Guides Strands Cards Meshes Total  Curv Name"), FontSilver); Newline(Ctx);
			LOOP
			for (uint It = 0; It < ComponentCount; ++It)
			{
				const FComponent Info = GetComponent(It);
				const uint Total
					= Info.GPU_Guides
					+ Info.GPU_Strands
					+ Info.GPU_Cards;


				PrintGroup(Ctx, Info.GroupIndex, Info.GroupCount);
	
				PrintLargeNumber(Ctx, Info.GPU_Guides,	FontWhite);		Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Strands,	FontWhite);		Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Cards,	FontWhite);		Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Meshes,	FontWhite);		Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Total,			FontOrange);	Print(Ctx, TEXT(" "), FontWhite);
				PrintCurve(Ctx, Info.ActiveCurve, Info.RestCurve);
	
				PrintComponentNames(Ctx, It, FontWhite);
				Newline(Ctx);
			}
		}
		Newline(Ctx);
	}

	// Grooms
	if (bGroom)
	{
		Print(Ctx, TEXT("Grooms"), FontYellow); Newline(Ctx);
		if (!bStrandsOnly)
		{
			Print(Ctx, TEXT("Grp. Guides Strands Cards Meshes Total Curv Name"), FontSilver); Newline(Ctx);
			LOOP
			for (uint It = 0; It < GroomCount; ++It)
			{
				const FGroom Info = GetGroom(It);
				PrintGroup(Ctx, Info.GroupIndex, Info.GroupCount);
	
				PrintLargeNumber(Ctx, Info.GPU_Guides, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Strands, FontWhite);Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Cards, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
				PrintLargeNumber(Ctx, Info.GPU_Meshes, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
				PrintCurve(Ctx, Info.ActiveCurve, Info.RestCurve);
				PrintGroomNames(Ctx, It, FontWhite);
				Newline(Ctx);
			}
		}
		else
		{
			FFontColor FontMemory = FontWhite;
			FFontColor FontCurve = FontSilver;

			Print(Ctx, TEXT("Grp. Rest.  Intrp. Cluster  RT.  Total  Curv Name"), FontSilver); Newline(Ctx);
			LOOP
			for (uint It = 0; It < GroomCount; ++It)
			{
				const FGroom Info = GetGroom(It);
				const uint GPUStrandsTotal 
					= Info.GPU_Rest
					+ Info.GPU_Interpolation
					+ Info.GPU_Cluster
					+ Info.GPU_Raytracing;

				PrintGroup(Ctx, Info.GroupIndex, Info.GroupCount);

				PrintLargeNumber(Ctx, Info.GPU_Rest, FontMemory);	Print(Ctx, TEXT(" "), FontMemory);
				PrintLargeNumber(Ctx, Info.GPU_Interpolation, FontMemory);Print(Ctx, TEXT(" "), FontMemory);
				PrintLargeNumber(Ctx, Info.GPU_Cluster, FontMemory);	Print(Ctx, TEXT(" "), FontMemory);
				PrintLargeNumber(Ctx, Info.GPU_Raytracing, FontMemory);	Print(Ctx, TEXT(" "), FontMemory);
				PrintLargeNumber(Ctx, GPUStrandsTotal, FontOrange);	Print(Ctx, TEXT(" "), FontMemory);
				PrintCurve(Ctx, Info.ActiveCurve, Info.RestCurve);
				PrintGroomNames(Ctx, It, FontMemory);
				Newline(Ctx);

				// Curve count detail per resource type
				#if 0
				Print(Ctx, TEXT("     "), FontCurve);
				PrintLargeNumber(Ctx, Info.Curve_Rest, FontCurve);	Print(Ctx, TEXT(" "), FontCurve);
				PrintLargeNumber(Ctx, Info.Curve_Interpolation, FontCurve);Print(Ctx, TEXT(" "), FontCurve);
				PrintLargeNumber(Ctx, Info.Curve_Cluster, FontCurve);	Print(Ctx, TEXT(" "), FontCurve);
				PrintLargeNumber(Ctx, Info.Curve_Raytracing, FontCurve);	Print(Ctx, TEXT(" "), FontCurve);
				Newline(Ctx);
				#endif
			}
		}
		Newline(Ctx);
	}

	// Bindings
	if (bBinding)
	{
		Print(Ctx, TEXT("Bindings"), FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Grp. Guides Strands Cards Meshes Total Curv Name"), FontSilver); Newline(Ctx);
		LOOP
		for (uint It = 0; It < BindingCount; ++It)
		{
			const FBinding Info = GetBinding(It);
			const uint Total
				= Info.GPU_Guides
				+ Info.GPU_Strands
				+ Info.GPU_Cards
				+ Info.GPU_Meshes;

			PrintGroup(Ctx, Info.GroupIndex, Info.GroupCount);

			PrintLargeNumber(Ctx, Info.GPU_Guides, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
			PrintLargeNumber(Ctx, Info.GPU_Strands, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
			PrintLargeNumber(Ctx, Info.GPU_Cards, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
			PrintLargeNumber(Ctx, Info.GPU_Meshes, FontWhite);	Print(Ctx, TEXT(" "), FontWhite);
			PrintLargeNumber(Ctx, Total, FontOrange);			Print(Ctx, TEXT(" "), FontWhite);
			PrintCurve(Ctx, Info.ActiveCurve, Info.RestCurve);
			PrintBindingNames(Ctx, It, FontWhite);
			Newline(Ctx);
		}
		Newline(Ctx);
	}
}

#endif // SHADER_PRINT_MEMORY

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// Debug draw clusters
#ifdef SHADER_CLUSTERAABB

#include "HairStrandsAABBCommon.ush"

uint InstanceRegisteredIndex;
uint ClusterOffset;
uint ClusterCount;
uint HairGroupId;
uint PointCount;
uint CurveCount;
uint bDrawAABB;

Buffer<int> GroupAABBBuffer;
Buffer<int> ClusterAABBBuffer;

[numthreads(GROUP_SIZE, 1, 1)]
void MainDrawDebugAABBCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x == 0)
	{
		const bool bDrawClusterInfo = true;
		const uint HairGroupHeight = 15;

		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50 + (HairGroupId == 0 ? 0 : HairGroupHeight) + HairGroupId * HairGroupHeight));

		FFontColor FontValue = FontWhite;
		FFontColor FontPercent = FontSilver;

		if (DispatchThreadId.x == 0)
		{
			if (HairGroupId == 0)
			{
				Print(Context, TEXT("Reg.Idx Clusters Curves Points"), FontYellow);
				Newline(Context);
			}
			Print(Context, InstanceRegisteredIndex, FontOrange, 8, 0);
			Print(Context, ClusterCount, FontValue, 9, 0);
			PrintLargeNumber(Context, CurveCount, FontValue);
			Print(Context, TEXT(" "), FontYellow);
			PrintLargeNumber(Context, PointCount, FontValue);
		}
	}

	if (bDrawAABB)
	{
		// Group
		if (DispatchThreadId.x == 0)
		{
			const FHairAABB GroupBound = ReadHairAABB(InstanceRegisteredIndex, GroupAABBBuffer);
			AddAABBTWS(GroupBound.Min, GroupBound.Max, float4(1.0f, 0.5f, 0.0f, 1.0f));
		}
	
		// Cluster
		if (DispatchThreadId.x < ClusterCount)
		{
			const FHairAABB ClusterBound = ReadHairAABB(ClusterOffset + DispatchThreadId.x, ClusterAABBBuffer, 1);
			AddAABBTWS(ClusterBound.Min, ClusterBound.Max, float4(0.5f, 1.0f, 0.0f, 0.3f));
		}
	}
}

#endif // SHADER_CLUSTERAABB

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/// Print skin cache stats
#if SHADER_PRINT_HAIR_SKIN_CACHE

uint			HairSkinCacheEnable;
uint			GPUSkinCacheEnable;
uint			InstanceCount;
uint			UniqueMeshCount;
Buffer<uint>	Infos;

FSTRINGS(UniqueMeshNames)

struct FInstanceInfo
{
	uint InstanceIndex;
	uint GeometryType;
	uint CacheType;
	uint InstanceCount;
	uint ActiveSectionCount;
	uint TotalSectionCount;
	uint LOD;
	uint Pad;
};

FInstanceInfo ReadInstanceInfo(uint Index)
{
	const uint Index8 = Index * 8u;

	uint4 Data0 = 0;
	Data0.x = Infos[Index8 + 0];
	Data0.y = Infos[Index8 + 1];
	Data0.z = Infos[Index8 + 2];
	Data0.w = Infos[Index8 + 3];

	uint4 Data1 = 0;
	Data1.x = Infos[Index8 + 4];
	Data1.y = Infos[Index8 + 5];
	Data1.z = Infos[Index8 + 6];
	Data1.w = Infos[Index8 + 7];

	FInstanceInfo Out = (FInstanceInfo)0;
	Out.InstanceIndex		= Data0.x;
	Out.GeometryType		= Data0.y;
	Out.CacheType			= Data0.z;
	Out.InstanceCount		= Data0.w;
	Out.ActiveSectionCount	= Data1.x;
	Out.TotalSectionCount	= Data1.y;
	Out.LOD					= Data1.z;
	Out.Pad					= Data1.w;
	return Out;
}

[numthreads(1, 1, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw AABB of the instance groups
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(50, 100));

	// Common
	bool bAdvanced = false;
	{
		Print(Ctx, TEXT("Instance count    : "), FontWhite);
		Print(Ctx, InstanceCount, FontYellow, 8);
		Newline(Ctx);
	
		Print(Ctx, TEXT("Unique mesh count : "), FontWhite);
		Print(Ctx, UniqueMeshCount, FontYellow, 8);
		Newline(Ctx);
	
		Print(Ctx, TEXT("GPU  skin cache   : "), FontWhite);
		PrintBool(Ctx, GPUSkinCacheEnable);
		Newline(Ctx);

		Print(Ctx, TEXT("Hair skin cache   : "), FontWhite);
		PrintBool(Ctx, HairSkinCacheEnable);
		Newline(Ctx);

		bAdvanced = AddCheckbox(Ctx, TEXT("Advanced"), false, FontSilver); 
		Newline(Ctx);
	}
	Newline(Ctx);

	// Draw instance details
	if (bAdvanced)
	{
		Print(Ctx, TEXT("  Inst. Guide Strands Cards CacheType Sections LOD Name"));
		Newline(Ctx);
	}

	LOOP
	for (uint UniqueMeshIndex = 0; UniqueMeshIndex < UniqueMeshCount; ++UniqueMeshIndex)
	{
		const FInstanceInfo Info = ReadInstanceInfo(UniqueMeshIndex);
		if (!bAdvanced && Info.ActiveSectionCount == 0)
		{
			continue;
		}

		// Index
		Print(Ctx,UniqueMeshIndex, FontSilver, 2);
		Print(Ctx, TEXT("  "));

		// Instance count
		Print(Ctx,Info.InstanceCount, FontYellow, 3, 0);
		Print(Ctx, TEXT(" "));

		// Geometry type
		if (Info.GeometryType & 1) { Print(Ctx, TEXT("x     "), FontGreen); } else { Print(Ctx, TEXT(".     "), FontRed); }
		if (Info.GeometryType & 2) { Print(Ctx, TEXT("x       "), FontGreen); } else { Print(Ctx, TEXT(".       "), FontRed); }
		if (Info.GeometryType & 4) { Print(Ctx, TEXT("x     "), FontGreen); } else { Print(Ctx, TEXT(".     "), FontRed); }

		// Cache type
		     if (Info.CacheType == 1){ Print(Ctx, TEXT("Skin"), FontEmerald); }
		else if (Info.CacheType == 2){ Print(Ctx, TEXT("Hair"), FontEmerald); }
		else if (Info.CacheType == 4){ Print(Ctx, TEXT("Geom"), FontEmerald); }
		else                         { Print(Ctx, TEXT("Unkw"), FontRed); }
		Print(Ctx, TEXT("      "));


		// Section
		Print(Ctx,Info.ActiveSectionCount, FontYellow, 3, 0);
		Print(Ctx, TEXT("/"));
		Print(Ctx,Info.TotalSectionCount, FontYellow, 3, 0);
		Print(Ctx, TEXT("    "));

		// LOD
		Print(Ctx,Info.LOD, FontYellow, 2, 0);

		// Name
		PrintUniqueMeshNames(Ctx, UniqueMeshIndex, FontWhite);

		Newline(Ctx);
	}
	Newline(Ctx);
}

#endif // SHADER_PRINT_HAIR_SKIN_CACHE