// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FDebugShadingInfo
{
	float3	BaseColor;
	float	Roughness;
	float3	T;
	uint	SampleCount;
	float3	V;
	uint	SampleOffset;
};

struct FDebugSample
{
	float3	Direction;
	float	Pdf;
	float3	Weight;
	float	Pad;
};