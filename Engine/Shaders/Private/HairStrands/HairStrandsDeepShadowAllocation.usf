// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Matrices.ush"
#include "HairStrandsDeepShadowCommonStruct.ush"
#include "HairStrandsAABBCommon.ush"

#if SHADER_ALLOCATE

#ifndef MAX_SLOT_COUNT
#error MAX_SLOT_COUNT needs to be defined
#endif

////////////////////////////////////////////////////////////////////////////////

float4x4 ComputeTranslatedWorldToLight(
	const FHairAABB TranslatedAABB,
	const float3 LightDirection,
	const float3 TranslatedLightPosition,
	const bool bIsDirectional)
{
	const float3 Extents = GetExtents(TranslatedAABB);
	const float3 Center  = GetCenter(TranslatedAABB);
	const float  Radius  = length(Extents);

	if (bIsDirectional) // (LightType == LightType_Directional)
	{
		return LookAtMatrix(Center - LightDirection * Radius, Center, float3(0, 0, 1));
	}
	else // if (LightType == LightType_Spot || LightType == LightType_Point || LightType == LightType_Rect)
	{
		return LookAtMatrix(TranslatedLightPosition, Center, float3(0, 0, 1));
	}
}

////////////////////////////////////////////////////////////////////////////////

// Mirror of FLightData in HairStrands/HairStrandsDeepShadow.cpp
struct FLightData
{
	float3 LightDirection;
	uint MacroGroupId;
	float3 TranslatedLightPosition;
	uint bIsLightDirectional;
};

float				RasterizationScale;
float				AABBScale;
float				MaxHafFovInRad;

int2				SlotResolution;
uint				SlotIndexCount;
uint				MacroGroupCount;

uint2				AtlasResolution;
float2				AtlasTexelSize;
uint				MinAtlasTileResolution;
uint				MinAtlasTileResolutionLog2;

StructuredBuffer<FLightData> 			LightDataBuffer;
Buffer<int>								MacroGroupAABBBuffer;
RWStructuredBuffer<FDeepShadowViewInfo>	OutShadowViewInfoBuffer;

// Each bin corresponds to a power of two shadow atlas tile resolution. The resolution is calculated as pow(2, MinAtlasTileResolutionLog2 + BinIndex).
// This gives us a maximum tile resolution of pow(2, MinAtlasTileResolutionLog2 + NUM_ATLAS_TILE_BINS - 1).
#define NUM_ATLAS_TILE_BINS 6
groupshared uint AtlasTileBinElementCounts[NUM_ATLAS_TILE_BINS];
groupshared uint AtlasTileBins[NUM_ATLAS_TILE_BINS][MAX_SLOT_COUNT];
groupshared uint4 AtlasTileAllocations[MAX_SLOT_COUNT];
groupshared uint AtlasTileDesiredResolutions[MAX_SLOT_COUNT];

float DegreesToRadians(float InDeg)
{
	return InDeg / 180.f * PI;
}

float ComputeMinStrandRadiusAtDepth1(const int2 Resolution, const float InFOVInRad, const float InRasterizationScale)
{
	const float DiameterToRadius = 0.5f;
	const float vFOV = InFOVInRad;
	const float StrandDiameterAtDepth1 = tan(vFOV * 0.5f) / (0.5f * Resolution.y);
	return DiameterToRadius * InRasterizationScale * StrandDiameterAtDepth1;
}

// This function is similar to the CPU version in HairStrandsDeepShadow.cpp
void ComputeTranslatedWorldToLightClip(
	inout float4x4	OutTranslatedWorldToClipTransform,
	inout float		OutMinStrandRadiusAtDepth1,
	inout float3	OutLightDirection,
	const FHairAABB TranslatedWorld_AABB,
	const FLightData LightData)
{
	float3 Center = GetCenter(TranslatedWorld_AABB);
	OutLightDirection = LightData.bIsLightDirectional ? LightData.LightDirection : -normalize(LightData.TranslatedLightPosition - Center);

	const float4x4 Coarse_TranslatedWorldToLight = ComputeTranslatedWorldToLight(TranslatedWorld_AABB, LightData.LightDirection, LightData.TranslatedLightPosition, LightData.bIsLightDirectional);

	const FHairAABB Light_AABB = Transform(TranslatedWorld_AABB, Coarse_TranslatedWorldToLight);
	const float3 Light_Extents = GetExtents(Light_AABB);

	const float Radius = length(GetExtents(TranslatedWorld_AABB)) * AABBScale;
	float MinZ = -Light_Extents.z * AABBScale;
	float MaxZ = +Light_Extents.z * AABBScale;

	const float StrandHairStableRasterizationScale = max(RasterizationScale, 1.0f);
	OutMinStrandRadiusAtDepth1 = 1;
	OutTranslatedWorldToClipTransform = 0;

	if (LightData.bIsLightDirectional)
	{
		const float4x4 TranslatedWorldToLight	= LookAtMatrix(Center - LightData.LightDirection * abs(MinZ), Center, float3(0, 0, 1));
		const float4x4 ProjMatrix				= ReversedZOrthoMatrix(Radius, Radius, 1.f / (MaxZ-MinZ), 0);
		OutTranslatedWorldToClipTransform		= mul(TranslatedWorldToLight, ProjMatrix);

		const float RadiusAtDepth1				= Radius / min(SlotResolution.x, SlotResolution.y);
		OutMinStrandRadiusAtDepth1				= RadiusAtDepth1 * RasterizationScale;
	}
	else // if (LightType == LightType_Spot || LightType == LightType_Point || LightType == LightType_Rect)
	{
		const float LightDistanceToCenter = length(LightData.TranslatedLightPosition - Center);
		MaxZ = max(0.2f, LightDistanceToCenter) + MaxZ;
		MinZ = max(0.1f, LightDistanceToCenter) + MinZ;
		MinZ = max(1.0f, MinZ);

		const float SphereDistance			= length(LightData.TranslatedLightPosition - Center);
		const float HalfFov					= min(MaxHafFovInRad, atan(Radius / SphereDistance));

		const float4x4 TranslatedWorldToLight	= LookAtMatrix(LightData.TranslatedLightPosition, Center, float3(0, 0, 1));
		const float4x4 ProjMatrix				= ReversedZPerspectiveMatrix(HalfFov, 1, 1, MinZ, MaxZ);
		OutTranslatedWorldToClipTransform		= mul(TranslatedWorldToLight, ProjMatrix);

		OutMinStrandRadiusAtDepth1 = ComputeMinStrandRadiusAtDepth1(SlotResolution, 2 * HalfFov, RasterizationScale);
	}
}

float2 ComputeProjectedScreenSize(float4x4 TranslatedWorldToLightClipTransform)
{
	float4x4 LightClipToTranslatedWorld = Inverse(TranslatedWorldToLightClipTransform);
	float4x4 LightClipToScreenClip = mul(LightClipToTranslatedWorld, View.TranslatedWorldToClip);
	float2 ProjectedMinUV = 99999.0f;
	float2 ProjectedMaxUV = -99999.0f;

	UNROLL
	for (int Y = 0; Y < 2; ++Y)
	{
		UNROLL
		for (int X = 0; X < 2; ++X)
		{
			float4 LightClip = float4(X * 2.0f - 1.0f, Y * 2.0f - 1.0f, 0.0f, 1.0f);
			float4 ScreenProjected = mul(LightClip, LightClipToScreenClip);
			float2 ScreenProjectedUV = (ScreenProjected.xy / ScreenProjected.w) * float2(0.5f, -0.5f) + 0.5f;

			ProjectedMinUV = min(ProjectedMinUV, ScreenProjectedUV);
			ProjectedMaxUV = max(ProjectedMaxUV, ScreenProjectedUV);
		}
	}

	float2 ProjectedSize = max(ProjectedMaxUV - ProjectedMinUV, 0.0f) * View.ViewSizeAndInvSize.xy;
	return ProjectedSize;
}

void AddToAtlasTileBin(float2 ProjectedShadowSize, uint SlotIndex)
{
	float ProjectedShadowSizeMaxDim = ceil(max(ProjectedShadowSize.x, ProjectedShadowSize.y));
	// Get the log2 of the next greater power of two.
	uint RequestedShadowSizeLog2 = ProjectedShadowSizeMaxDim > 0.0f ? (uint)ceil(log2(ProjectedShadowSizeMaxDim)) : 0;
	RequestedShadowSizeLog2 = clamp(RequestedShadowSizeLog2, MinAtlasTileResolutionLog2, (MinAtlasTileResolutionLog2 + NUM_ATLAS_TILE_BINS - 1));
	uint BinIndex = RequestedShadowSizeLog2 - MinAtlasTileResolutionLog2;

	// Append to bin
	uint WriteIndex = 0;
	InterlockedAdd(AtlasTileBinElementCounts[BinIndex], 1, WriteIndex);
	// Note: Currently we have MAX_SLOT_COUNT == FHairStrandsDeepShadowData::MaxMacroGroupCount, 
	// so by sizing the AtlasTileBins arrays to MAX_SLOT_COUNT we can't overflow. If we ever change that,
	// we need to account for this here.
	AtlasTileBins[BinIndex][WriteIndex] = SlotIndex;
	
	// Keep the actual desired resolution around
	AtlasTileDesiredResolutions[SlotIndex] = max(MinAtlasTileResolution, (uint)ProjectedShadowSizeMaxDim);
}

void AllocateAtlasTiles()
{
	uint MaxAllowedTileResolution = 1u << (MinAtlasTileResolutionLog2 + NUM_ATLAS_TILE_BINS - 1);

	// Allocating might fail when the requested tiles are too big, so we retry allocating with smaller tiles when that happens.
	for (uint NumRetries = 0; NumRetries < NUM_ATLAS_TILE_BINS; ++NumRetries)
	{
		uint2 CurrentOffset = 0;
		uint CurrentRowHeight = 0;
		// Keep track of the actual maximum tile resolution requested in this allocation attempt
		uint MaxActualTileResolution = 0;

		// Loop over all bins starting with the one with the highest resolution
		for (uint i = 0; i < NUM_ATLAS_TILE_BINS; ++i)
		{
			const uint BinIndex = NUM_ATLAS_TILE_BINS - 1 - i;
			const uint NumElements = AtlasTileBinElementCounts[BinIndex];

			// Skip empty bins
			if (NumElements == 0)
			{
				continue;
			}

			uint BinResolution = 1u << (BinIndex + MinAtlasTileResolutionLog2);
			BinResolution = min(BinResolution, MaxAllowedTileResolution); // Start clamping the resolution of larger tiles when previous allocations did not succeed
			MaxActualTileResolution = max(MaxActualTileResolution, BinResolution);

			// Allocate space for all elements in this bin
			for (uint ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
			{
				const uint SlotIndex = AtlasTileBins[BinIndex][ElementIndex];
				const uint DesiredResolution = AtlasTileDesiredResolutions[SlotIndex];
				const uint ActualResolution = min(DesiredResolution, BinResolution);

				// Advance to the next row if we reached the right end of the atlas
				if ((CurrentOffset.x + ActualResolution) > AtlasResolution.x)
				{
					CurrentOffset.x = 0;
					CurrentOffset.y += CurrentRowHeight;
					CurrentRowHeight = 0;
				}

				AtlasTileAllocations[SlotIndex] = uint4(CurrentOffset, ActualResolution.xx);
				CurrentOffset.x += ActualResolution;
				CurrentRowHeight = max(CurrentRowHeight, ActualResolution);
			}
		}

		if ((CurrentOffset.y + CurrentRowHeight) <= AtlasResolution.y)
		{
			break;
		}

		// Half the maximum allowed tile resolution with each failed attempt.
		// Note that we half the actual maximum requested resolution which allows us to skip certain resolutions where the corresponding bins are empty.
		MaxAllowedTileResolution = MaxActualTileResolution / 2;
	}
}

void ApplyAtlasTileScaleBias(float4x4 ShadowMatrix, uint SlotIndex, out float4x4 ModifiedTranslatedWorldToClip, out float4 AtlasScaleBias)
{
	uint4 TileOffsetSize = AtlasTileAllocations[SlotIndex];
	float2 Scale = TileOffsetSize.zw * AtlasTexelSize;
	float2 Bias = TileOffsetSize.xy * AtlasTexelSize;

	AtlasScaleBias = float4(Scale, Bias);

#if 1
	Bias = Bias * float2(2.0f, -2.0f) - float2(1.0f, -1.0f) + float2(Scale.x, -Scale.y);

	float4x4 ScaleBias = float4x4(
		Scale.x, 0.0f, 0.0f, 0.0f,
		0.0f, Scale.y, 0.0f, 0.0f,
		0.0f, 0.0f, 1.0f, 0.0f,
		Bias.x, Bias.y, 0.0f, 1.0f
	);

	ModifiedTranslatedWorldToClip = mul(ShadowMatrix, ScaleBias);
#else
	float4x4 ClipToUV = float4x4(
		0.5f, 0.0f, 0.0f, 0.0f,
		0.0f, -0.5f, 0.0f, 0.0f,
		0.0f, 0.0f, 1.0f, 0.0f,
		0.5f, 0.5f, 0.0f, 1.0f
	);

	float4x4 UVToClip = float4x4(
		2.0f, 0.0f, 0.0f, 0.0f,
		0.0f, -2.0f, 0.0f, 0.0f,
		0.0f, 0.0f, 1.0f, 0.0f,
		-1.0f, 1.0f, 0.0f, 1.0f
	);
	
	float4x4 ScaleBias = float4x4(
		Scale.x, 0.0f, 0.0f, 0.0f,
		0.0f, Scale.y, 0.0f, 0.0f,
		0.0f, 0.0f, 1.0f, 0.0f,
		Bias.x, Bias.y, 0.0f, 1.0f
	);

	ModifiedTranslatedWorldToClip = mul(ShadowMatrix, mul(ClipToUV, mul(ScaleBias, UVToClip)));
#endif
}

// This code assume we have less than 32 macro group (which fit into a single CU/SM)
[numthreads(MAX_SLOT_COUNT, 1, 1)]
void CreateViewInfo(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint SlotIndex = DispatchThreadId.x;

	// Clear LDS
	{
		UNROLL
		for (uint WaveOffset = 0; WaveOffset < NUM_ATLAS_TILE_BINS; WaveOffset += MAX_SLOT_COUNT)
		{
			uint BinIndex = WaveOffset + SlotIndex;
			if (BinIndex < NUM_ATLAS_TILE_BINS)
			{
				AtlasTileBinElementCounts[BinIndex] = 0;
			}
		}
	}
	GroupMemoryBarrierWithGroupSync();

	FDeepShadowViewInfo ViewInfo = (FDeepShadowViewInfo)0;
	if (SlotIndex < SlotIndexCount)
	{
		const FLightData LightData = LightDataBuffer[SlotIndex];

		FHairAABB TranslatedBound = InitHairAABB();
		if (LightData.MacroGroupId < MacroGroupCount)
		{
			TranslatedBound = ReadHairAABB(LightData.MacroGroupId, MacroGroupAABBBuffer);
		}

		ComputeTranslatedWorldToLightClip(ViewInfo.TranslatedWorldToClip, ViewInfo.MinRadiusAtDepth1, ViewInfo.ViewForward, TranslatedBound, LightData);

		float2 ProjectedShadowSize = ComputeProjectedScreenSize(ViewInfo.TranslatedWorldToClip);
		AddToAtlasTileBin(ProjectedShadowSize, SlotIndex);
	}

	GroupMemoryBarrierWithGroupSync();
	if (SlotIndex == 0)
	{
		AllocateAtlasTiles();
	}
	GroupMemoryBarrierWithGroupSync();

	if (SlotIndex < SlotIndexCount)
	{
		ApplyAtlasTileScaleBias(ViewInfo.TranslatedWorldToClip, SlotIndex, ViewInfo.TranslatedWorldToClipScaledBiased, ViewInfo.AtlasScaleBias);
		OutShadowViewInfoBuffer[SlotIndex] = ViewInfo;
	}
}
#endif
