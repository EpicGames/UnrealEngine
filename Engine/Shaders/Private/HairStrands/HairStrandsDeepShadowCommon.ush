// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "../ColorMap.ush"  

#define DEBUG_MODE 0

// Note: in the following functions, InLayerDepths are Layers' depths in clip space (prior to inverse Z)

bool IsDepthCloser(float a, float b)
{
	return a < b;
}

float ComputeDOMWeight(float DistanceToFrontDepth, float LayerDepth)
{
	return IsDepthCloser(DistanceToFrontDepth, LayerDepth) ? 1 : 0;
}

float4 ComputeDOMWeights(float DistanceToFrontDepth, float4 InLayerDepths)
{
	float4 Weigths = 0;
	Weigths[0] = ComputeDOMWeight(DistanceToFrontDepth, InLayerDepths[0]);
	Weigths[1] = ComputeDOMWeight(DistanceToFrontDepth, InLayerDepths[1]);
	Weigths[2] = ComputeDOMWeight(DistanceToFrontDepth, InLayerDepths[2]);
	Weigths[3] = ComputeDOMWeight(DistanceToFrontDepth, InLayerDepths[3]);
	return Weigths;
}	

float3 HairDebugColor(float DistanceToFrontDepth, float4 InLayerDepths)
{
	float3 color = 0;
	if (DistanceToFrontDepth < InLayerDepths[3]) color = float3(0, 0, 1);
	if (DistanceToFrontDepth < InLayerDepths[2]) color = float3(0, 1, 0);
	if (DistanceToFrontDepth < InLayerDepths[1]) color = float3(1, 1, 0);
	if (DistanceToFrontDepth < InLayerDepths[0]) color = float3(1,0,0);
	return color;
}

float InterpolateCount(float DepthToFrontDepth, float Layer0Depth, float Layer1Depth)
{
	return saturate((DepthToFrontDepth - Layer0Depth) / (Layer1Depth - Layer0Depth));
}

float ComputeHairCount(float4 DomValue, float DistanceToFrontDepth, float4 InLayerDepths)
{
	float OutCount = 0;
	
	if (DistanceToFrontDepth < InLayerDepths[0])
		OutCount = lerp(          0, DomValue[0], InterpolateCount(DistanceToFrontDepth, 0, InLayerDepths[0]));
	else if (DistanceToFrontDepth < InLayerDepths[1])
		OutCount = lerp(DomValue[0], DomValue[1], InterpolateCount(DistanceToFrontDepth, InLayerDepths[0], InLayerDepths[1]));
	else if (DistanceToFrontDepth < InLayerDepths[2])
		OutCount = lerp(DomValue[1], DomValue[2], InterpolateCount(DistanceToFrontDepth, InLayerDepths[1], InLayerDepths[2]));
	else if (DistanceToFrontDepth < InLayerDepths[3])
		OutCount = lerp(DomValue[2], DomValue[3], InterpolateCount(DistanceToFrontDepth, InLayerDepths[2], InLayerDepths[3]));
	else 
		OutCount = DomValue[3];
	return OutCount;
}

float GetDomDistanceToFrontDepth(float FrontDepth, float LightSpaceZ)
{
	return max(0.0f, FrontDepth - LightSpaceZ);
}

float GetDomDistanceToFrontDepthWithBias(float FrontDepth, float LightSpaceZ, float DepthBias)
{
	return max(0.0f, FrontDepth - LightSpaceZ - DepthBias);
}

float SampleDOM_PCF2x2(
	float3 LightSpacePosition, 
	float DepthBias,
	const float4 InLayerDepths,
	Texture2D<float> FrontDepthTexture,
	Texture2D<float4> DomTexture)
{
	// Find the bottom left corner texel for bilinear interpolation
	const float2 PBottomLeft = floor(LightSpacePosition.xy - float(0.5f).xx); // Bottom-left corner of the bottom left texel.
	const uint2 C0 = uint2(PBottomLeft);
	const uint2 C1 = C0 + uint2(1,0);
	const uint2 C2 = C0 + uint2(0,1);
	const uint2 C3 = C0 + uint2(1,1);
	
	// @todo_hair: gather4
	const float FrontDepth0 = FrontDepthTexture.Load(uint3(C0, 0)).x;
	const float FrontDepth1 = FrontDepthTexture.Load(uint3(C1, 0)).x;
	const float FrontDepth2 = FrontDepthTexture.Load(uint3(C2, 0)).x;
	const float FrontDepth3 = FrontDepthTexture.Load(uint3(C3, 0)).x;

	const float4 DOMValue0  = DomTexture.Load(uint3(C0, 0));
	const float4 DOMValue1  = DomTexture.Load(uint3(C1, 0));
	const float4 DOMValue2  = DomTexture.Load(uint3(C2, 0));
	const float4 DOMValue3  = DomTexture.Load(uint3(C3, 0));
	
	const float HairCount0	= ComputeHairCount(DOMValue0, GetDomDistanceToFrontDepthWithBias(FrontDepth0, LightSpacePosition.z, DepthBias), InLayerDepths);
	const float HairCount1	= ComputeHairCount(DOMValue1, GetDomDistanceToFrontDepthWithBias(FrontDepth1, LightSpacePosition.z, DepthBias), InLayerDepths);
	const float HairCount2	= ComputeHairCount(DOMValue2, GetDomDistanceToFrontDepthWithBias(FrontDepth2, LightSpacePosition.z, DepthBias), InLayerDepths);
	const float HairCount3	= ComputeHairCount(DOMValue3, GetDomDistanceToFrontDepthWithBias(FrontDepth3, LightSpacePosition.z, DepthBias), InLayerDepths);

	const float2 S = frac(LightSpacePosition.xy - (PBottomLeft + 0.5f));
	const float HairCount01 = lerp(HairCount0, HairCount1, S.x);
	const float HairCount23 = lerp(HairCount2, HairCount3, S.x);
	return lerp(HairCount01, HairCount23, S.y);
}

float SampleDOM_PCF(
	float3 LightSpacePosition, 
	float DepthBias,
	const float4 InLayerDepths,
	Texture2D<float> FrontDepthTexture,
	Texture2D<float4> DomTexture)
{	
	// Explicit 1 ring (6x6 / 5x5) with uniform weighting
	float HairCount = 0;
	float w = 1;
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3(-2,-2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 0,-2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 2,-2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);

	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3(-2, 0, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 0, 0, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 2, 0, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);

	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3(-2, 2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 0, 2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	HairCount += w * SampleDOM_PCF2x2(LightSpacePosition + float3( 2, 2, 0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);

	HairCount /= w * 9;
	return HairCount;
}

// Sample [-1,1]
// Jitter [0,1]
float2 ComputeJitteredSample(float2 Sample, float2 Jitter)
{
#if 0
	const float2 NormSample = (Sample + float2(1, 1)) * 0.5f;
	const float2 JitteredNormSample = frac(NormSample + Jitter);
	return JitteredNormSample *2 - float2(1, 1);
#else
	return Sample;
#endif
}

float SampleDOM_PCSS(
	float3 LightSpacePosition,
	uint2 DeepShadowAtlasResolution,
	float DepthBias,
	float4 InLayerDepths,
	Texture2D<float> FrontDepthTexture,
	Texture2D<float4> DomTexture,
	float ApexAngleInDegree,
	float2 Jitter) // [0,1]
{
	// Poisson disk position http://developer.download.nvidia.com/whitepapers/2008/PCSS_Integration.pdf
	float2 PoissonDisk[16] =
	{
		 float2(-0.94201624, -0.39906216),
		 float2(0.94558609, -0.76890725),
		 float2(-0.094184101, -0.92938870),
		 float2(0.34495938, 0.29387760),
		 float2(-0.91588581, 0.45771432),
		 float2(-0.81544232, -0.87912464),
		 float2(-0.38277543, 0.27676845),
		 float2(0.97484398, 0.75648379),
		 float2(0.44323325, -0.97511554),
		 float2(0.53742981, -0.47373420),
		 float2(-0.26496911, -0.41893023),
		 float2(0.79197514, 0.19090188),
		 float2(-0.24188840, 0.99706507),
		 float2(-0.81409955, 0.91437590),
		 float2(0.19984126, 0.78641367),
		 float2(0.14383161, -0.14100790)
	};

	// Find the closest occluder from light point of view
	const uint OccluderCount = 5;
	float OccluderDistance = 0; // Occluder distance in clip space
	for (uint OccluderIt = 0; OccluderIt < OccluderCount; ++OccluderIt)
	{
		const float2 Offset = ComputeJitteredSample(PoissonDisk[OccluderIt], Jitter) * DeepShadowAtlasResolution;
		const float2 SamplePosition = LightSpacePosition.xy + Offset;
		const float2 P = floor(SamplePosition - float(0.5f).xx);
		const uint2 C0 = P;
		const uint2 C1 = C0 + uint2(1, 0);
		const uint2 C2 = C0 + uint2(0, 1);
		const uint2 C3 = C0 + uint2(1, 1);

		// @todo_hair: gather4
		const float FrontDepth0 = FrontDepthTexture.Load(uint3(C0, 0)).x;
		const float FrontDepth1 = FrontDepthTexture.Load(uint3(C1, 0)).x;
		const float FrontDepth2 = FrontDepthTexture.Load(uint3(C2, 0)).x;
		const float FrontDepth3 = FrontDepthTexture.Load(uint3(C3, 0)).x;

		const float Distance0 = GetDomDistanceToFrontDepth(FrontDepth0, LightSpacePosition.z);
		const float Distance1 = GetDomDistanceToFrontDepth(FrontDepth1, LightSpacePosition.z);
		const float Distance2 = GetDomDistanceToFrontDepth(FrontDepth2, LightSpacePosition.z);
		const float Distance3 = GetDomDistanceToFrontDepth(FrontDepth3, LightSpacePosition.z);

		OccluderDistance = max(OccluderDistance, max(Distance0, max(Distance1, max(Distance2, Distance3))));
	}

	// Sample DOM
	float HairCount = 0;
	const uint SampleCount = 16;
	const float ApexAngleInRad = ApexAngleInDegree / 180.f * 3.141592653f;
	const float ApexHalfAngleInRad = ApexAngleInRad * 0.5f;
	const float SampleRadius = OccluderDistance * tan(ApexHalfAngleInRad); // Sample radius in clip space
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float2 Offset = ComputeJitteredSample(PoissonDisk[SampleIt], Jitter) * SampleRadius * DeepShadowAtlasResolution;
		HairCount += SampleDOM_PCF2x2(LightSpacePosition + float3(Offset,0), DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
	}
	
	HairCount /= SampleCount;
	return HairCount;
}

float3 ToLightPosition(float3 WorldPosition, float4x4 WorldToLightTransform)
{
	float4 LightPos = mul(float4(WorldPosition, 1), WorldToLightTransform);
	LightPos.xyz /= LightPos.w;
	const float2 LightUV = (LightPos.xy + float(1).xx) * 0.5f;
	return float3(LightUV.x, 1 - LightUV.y, LightPos.z);
}

float3 ComputeDomDebugColor(
	float3 WorldPosition, 
	float4x4 WorldToLightTransform, 
	float4 InLayerDepths,
	Texture2D<float> FrontDepthTexture, 
	SamplerState LinearSampler)
{
	const float3 LightSpacePosition = ToLightPosition(WorldPosition, WorldToLightTransform); 
	const float FrontDepth = FrontDepthTexture.SampleLevel(LinearSampler, LightSpacePosition.xy, 0).x;
	const float DistanceToFrontDepth = GetDomDistanceToFrontDepth(FrontDepth, LightSpacePosition.z);
	return HairDebugColor(DistanceToFrontDepth, InLayerDepths);
}

float3 ComputeHairCountDebugColor(
	float3 WorldPosition, 
	float4x4 WorldToLightTransform, 
	float4 InLayerDepths,
	Texture2D<float> FrontDepthTexture, 
	Texture2D<float4> DomTexture, 
	SamplerState LinearSampler, 
	float MaxHairCount)
{
	const float3 LightSpacePosition = ToLightPosition(WorldPosition, WorldToLightTransform);
	const float FrontDepth = FrontDepthTexture.SampleLevel(LinearSampler, LightSpacePosition.xy, 0).x;
	const float4 DOMValue = DomTexture.SampleLevel(LinearSampler, LightSpacePosition.xy, 0);
	const float DistanceToFrontDepth = GetDomDistanceToFrontDepth(FrontDepth, LightSpacePosition.z);
	const float HairCount = ComputeHairCount(DOMValue, DistanceToFrontDepth, InLayerDepths);
	return GetHSVDebugColor(HairCount / MaxHairCount);
}