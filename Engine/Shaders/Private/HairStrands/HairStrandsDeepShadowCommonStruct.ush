// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FDeepShadowViewInfo
{
	float4x4 TranslatedWorldToClipScaledBiased; // Has the shadow atlas tile scale bias baked in
	float4x4 TranslatedWorldToClip;
	float4 AtlasScaleBias; // Scale/bias for transforming UV coordinates into the shadow atlas tile of this shadow view.
	float3   ViewForward;
	float    MinRadiusAtDepth1;
};