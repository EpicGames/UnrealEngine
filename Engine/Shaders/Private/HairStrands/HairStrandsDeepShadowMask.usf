// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../MonteCarlo.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVoxelPageCommon.ush"
#include "HairStrandsCommon.ush"
#include "HairStrandsDeepShadowCommon.ush"
#include "HairStrandsDeepTransmittanceCommon.ush"

float4 WriteShadowMaskOutput(const float InFadedShadow, uint InEncodingType)
{
	// Depending of if the light is direction (i.e., WholeSceneLight) or a local light (Point/Spot/Rect), 
	// we change the output layout
	const float EncodedShadow = EncodeLightAttenuation(InFadedShadow);
	switch (InEncodingType)
	{
		/* Default    */case 0: return EncodedShadow;
		/* WholeLight */case 1: return float4(EncodedShadow, 1.f, 1.f, 1.f);
		/* LocalLight */case 2: return float4(1.f, 1.f, EncodedShadow, 1.f);
	}
	return EncodedShadow;
}

////////////////////////////////////////////////////////////////////////////////////////////////
// Deep shadow 

#define KERNEL_TYPE_2x2 0 
#define KERNEL_TYPE_4x4 1
#define KERNEL_TYPE_Gaussian8 2
#define KERNEL_TYPE_Gaussian16 3
#define KERNEL_TYPE_GaussianTransmission8 4

float3 GetTranslatedWorldPosition(float2 UV, float SceneDepth)
{
	const float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	return mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
}


#if SHADER_SHADOWMASK_DEEPSHADOW

StructuredBuffer<FDeepShadowViewInfo> DeepShadow_ViewInfoBuffer;
StructuredBuffer<uint> DeepShadow_AtlasSlotIndexBuffer;

int2				DeepShadow_AtlasResolution;
uint				DeepShadow_KernelType;
float				DeepShadow_DepthBiasScale;
float				DeepShadow_DensityScale;
float4				DeepShadow_LayerDepths;
float 				FadeAlpha;
uint				EncodingType;
uint				EffectiveAtlasSlotCount;

Texture2D<float>	DeepShadow_FrontDepthTexture;
Texture2D<float4>	DeepShadow_DomTexture;

SamplerState		LinearSampler;
SamplerState 		ShadowSampler;

// Return the light space position of a world space.
// XY: UV of the shadow space
//  Z: Normalized depth value in light clip space
//  W: Positive if the position is valid, negative otherwise
float4 ToLightPosition(float3 InTranslatedWorldPosition, uint InAtlasSlotIndex, out float4 AtlasScaleBias)
{
	FDeepShadowViewInfo ShadowViewInfo = DeepShadow_ViewInfoBuffer[InAtlasSlotIndex];
	float4x4 TranslatedWorldToLightTransform = ShadowViewInfo.TranslatedWorldToClip;
	AtlasScaleBias = ShadowViewInfo.AtlasScaleBias;

	float4 LightPos = mul(float4(InTranslatedWorldPosition,1), TranslatedWorldToLightTransform);
	LightPos.xyz/= LightPos.w;
	const float2 LightUV = (LightPos.xy + float(1).xx) * 0.5f;
	return float4(LightUV.x, 1-LightUV.y, LightPos.z, sign(LightPos.w));
}

bool IsCloser(float DepthA, float DepthB)
{
	// Inversed Z
	return DepthA > DepthB;
}

#define TILE_PIXEL_SIZE 8
#define OPAQUE_PCF 1

struct FHairSamplingSettings
{
	Texture2D<float>ShadowDepthTexture;
	Texture2D<float4>ShadowDomTexture;
	SamplerState	ShadowDepthTextureSampler;
	float2			ShadowAtlasInvResolution;
	float2			ShadowAtlasResolution;
	float2			ShadowSlotResolution;
	float2			ShadowSlotOffset;
	float			SceneDepth;					// SceneDepth in lightspace.
};

void InternalFetchRowOfThree(float2 Sample00TexelCenter, float VerticalOffset, inout float3 Values0, FHairSamplingSettings Settings)
{
	Values0.x = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(0, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0.y = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(1, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0.z = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(2, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0 = float3(Settings.SceneDepth < Values0);
}

void FetchRowOfFour(float2 Sample00TexelCenter, float VerticalOffset, inout float4 Values0, FHairSamplingSettings Settings)
{
	Values0.x = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(0, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0.y = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(1, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0.z = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(2, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0.w = Settings.ShadowDepthTexture.SampleLevel(Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(3, VerticalOffset)) * Settings.ShadowAtlasInvResolution, 0).r;
	Values0 = float4(Settings.SceneDepth < Values0);
}

float InternalPCF2x2(float2 Fraction, float3 Values0, float3 Values1, float3 Values2)
{
	float3 Results;

	Results.x = Values0.x * (1.0f - Fraction.x);
	Results.y = Values1.x * (1.0f - Fraction.x);
	Results.z = Values2.x * (1.0f - Fraction.x);
	Results.x += Values0.y;
	Results.y += Values1.y;
	Results.z += Values2.y;
	Results.x += Values0.z * Fraction.x;
	Results.y += Values1.z * Fraction.x;
	Results.z += Values2.z * Fraction.x;

	return saturate(0.25f * dot(Results, half3(1.0f - Fraction.y, 1.0f, Fraction.y)));
}

float InternalPCF3x3(float2 Fraction, float4 Values0, float4 Values1, float4 Values2, float4 Values3)
{
	float4 Results;

	Results.x = Values0.x * (1.0f - Fraction.x);
	Results.y = Values1.x * (1.0f - Fraction.x);
	Results.z = Values2.x * (1.0f - Fraction.x);
	Results.w = Values3.x * (1.0f - Fraction.x);
	Results.x += Values0.y;
	Results.y += Values1.y;
	Results.z += Values2.y;
	Results.w += Values3.y;
	Results.x += Values0.z;
	Results.y += Values1.z;
	Results.z += Values2.z;
	Results.w += Values3.z;
	Results.x += Values0.w * Fraction.x;
	Results.y += Values1.w * Fraction.x;
	Results.z += Values2.w * Fraction.x;
	Results.w += Values3.w * Fraction.x;

	return saturate(dot(Results, float4(1.0f - Fraction.y, 1.0f, 1.0f, Fraction.y)) * (1.0f / 9.0f));
}

float InternalManual2x2PCF(float2 ShadowPosition, FHairSamplingSettings Settings)
{
	float2 TexelPos = ShadowPosition * Settings.ShadowAtlasResolution;
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos) + 0.5f;
	float2 Sample00TexelCenter = TexelCenter - float2(1, 1);

	float3 SamplesValues0, SamplesValues1, SamplesValues2;
	InternalFetchRowOfThree(Sample00TexelCenter, 0, SamplesValues0, Settings);
	InternalFetchRowOfThree(Sample00TexelCenter, 1, SamplesValues1, Settings);
	InternalFetchRowOfThree(Sample00TexelCenter, 2, SamplesValues2, Settings);

	return InternalPCF2x2(Fraction, SamplesValues0, SamplesValues1, SamplesValues2);
}

float InternalManual4x4PCF(float2 ShadowPosition, FHairSamplingSettings Settings)
{
	float2 TexelPos = ShadowPosition * Settings.ShadowAtlasResolution - 0.5f;
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos) + 0.5f;	// bias to get reliable texel center content	
	float2 Sample00TexelCenter = TexelCenter - float2(1, 1);

	float4 SampleValues0, SampleValues1, SampleValues2, SampleValues3;
	FetchRowOfFour(Sample00TexelCenter, 0, SampleValues0, Settings);
	FetchRowOfFour(Sample00TexelCenter, 1, SampleValues1, Settings);
	FetchRowOfFour(Sample00TexelCenter, 2, SampleValues2, Settings);
	FetchRowOfFour(Sample00TexelCenter, 3, SampleValues3, Settings);
	return InternalPCF3x3(Fraction, SampleValues0, SampleValues1, SampleValues2, SampleValues3);
}

float2 InternalHorizontalPCF5x2(float2 Fraction, float4 Values00, float4 Values20, float4 Values40)
{
	float Results0;
	float Results1;

	Results0 = Values00.w * (1.0 - Fraction.x);
	Results1 = Values00.x * (1.0 - Fraction.x);
	Results0 += Values00.z;
	Results1 += Values00.y;
	Results0 += Values20.w;
	Results1 += Values20.x;
	Results0 += Values20.z;
	Results1 += Values20.y;
	Results0 += Values40.w;
	Results1 += Values40.x;
	Results0 += Values40.z * Fraction.x;
	Results1 += Values40.y * Fraction.x;

	return float2(Results0, Results1);
}

float4 InternalGather4(float2 SamplePos, int2 Offset, FHairSamplingSettings Settings)
{
	float4 Values = Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, Offset);
	return float4(
		Settings.SceneDepth < Values.x,
		Settings.SceneDepth < Values.y,
		Settings.SceneDepth < Values.z,
		Settings.SceneDepth < Values.w);
}

// high quality, 6x6 samples, using gather4
float InternalManual7x7PCF(float2 ShadowPosition, FHairSamplingSettings Settings)
{
#if 1
	float2 TexelPos = ShadowPosition * Settings.ShadowAtlasResolution - 0.5f;	// bias to be consistent with texture filtering hardware
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos);
	float2 SamplePos = (TexelCenter + 0.5f) / Settings.ShadowAtlasResolution;
#else
	float2 TexelPos = ShadowPosition * Settings.ShadowAtlasResolution;
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos) + 0.5f;
	float2 SamplePos = (TexelCenter - float2(1, 1)) / Settings.ShadowAtlasResolution;
#endif
	float Results;

	float4 Values00 = InternalGather4(SamplePos, int2(-2,-2), Settings);
	float4 Values20 = InternalGather4(SamplePos, int2( 0,-2), Settings);
	float4 Values40 = InternalGather4(SamplePos, int2( 2,-2), Settings);

	float2 Row0 = InternalHorizontalPCF5x2(Fraction, Values00, Values20, Values40);
	Results = Row0.x * (1.0f - Fraction.y) + Row0.y;

	float4 Values02 = InternalGather4(SamplePos, int2(-2,0), Settings);
	float4 Values22 = InternalGather4(SamplePos, int2( 0,0), Settings);
	float4 Values42 = InternalGather4(SamplePos, int2( 2,0), Settings);

	float2 Row1 = InternalHorizontalPCF5x2(Fraction, Values02, Values22, Values42);
	Results += Row1.x + Row1.y;

	float4 Values04 = InternalGather4(SamplePos, int2(-2,2), Settings);
	float4 Values24 = InternalGather4(SamplePos, int2( 0,2), Settings);
	float4 Values44 = InternalGather4(SamplePos, int2( 2,2), Settings);

	float2 Row2 = InternalHorizontalPCF5x2(Fraction, Values04, Values24, Values44);
	Results += Row2.x + Row2.y * Fraction.y;

	return 0.04f * Results;
}

float InternalGaussianFilter(float2 ShadowPosition, FHairSamplingSettings Settings, uint SampleCount)
{
	// Poisson disk position http://developer.download.nvidia.com/whitepapers/2008/PCSS_Integration.pdf
	float2 PoissonDisk[16] =
	{
			float2(-0.94201624, -0.39906216),
			float2(0.94558609, -0.76890725),
			float2(-0.094184101, -0.92938870),
			float2(0.34495938, 0.29387760),
			float2(-0.91588581, 0.45771432),
			float2(-0.81544232, -0.87912464),
			float2(-0.38277543, 0.27676845),
			float2(0.97484398, 0.75648379),
			float2(0.44323325, -0.97511554),
			float2(0.53742981, -0.47373420),
			float2(-0.26496911, -0.41893023),
			float2(0.79197514, 0.19090188),
			float2(-0.24188840, 0.99706507),
			float2(-0.81409955, 0.91437590),
			float2(0.19984126, 0.78641367),
			float2(0.14383161, -0.14100790)
	};

	const float SampleRadius = 3.f * Settings.ShadowAtlasInvResolution.x;
	float AccShadow = 0;
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float2 Offset = PoissonDisk[SampleIt] * SampleRadius;
		const float2 SamplePosition = ShadowPosition + Offset;
		AccShadow += InternalManual2x2PCF(SamplePosition, Settings);
	}

	AccShadow /= SampleCount;
	return AccShadow;
}

float InternalGaussianTransmissionFilter(float3 ShadowPosition, FHairSamplingSettings Settings, uint SampleCount)
{
	// Poisson disk position http://developer.download.nvidia.com/whitepapers/2008/PCSS_Integration.pdf
	float2 PoissonDisk[16] =
	{
			float2(-0.94201624, -0.39906216),
			float2(0.94558609, -0.76890725),
			float2(-0.094184101, -0.92938870),
			float2(0.34495938, 0.29387760),
			float2(-0.91588581, 0.45771432),
			float2(-0.81544232, -0.87912464),
			float2(-0.38277543, 0.27676845),
			float2(0.97484398, 0.75648379),
			float2(0.44323325, -0.97511554),
			float2(0.53742981, -0.47373420),
			float2(-0.26496911, -0.41893023),
			float2(0.79197514, 0.19090188),
			float2(-0.24188840, 0.99706507),
			float2(-0.81409955, 0.91437590),
			float2(0.19984126, 0.78641367),
			float2(0.14383161, -0.14100790)
	};

	const float SampleRadius = 3.f * Settings.ShadowAtlasInvResolution.x;
	float AccHairCount = 0;
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float2 Offset = PoissonDisk[SampleIt] * SampleRadius;
		const float DepthBias = DeepShadow_LayerDepths[0] * DeepShadow_DepthBiasScale;
		const float3 LightSpacePositionInAtlas = (ShadowPosition+float3(Offset,0)) * float3(DeepShadow_AtlasResolution, 1);
		const float HairCount = SampleDOM_PCF2x2(LightSpacePositionInAtlas.xyz, DepthBias, DeepShadow_LayerDepths, Settings.ShadowDepthTexture, Settings.ShadowDomTexture);
		AccHairCount += saturate(HairCount);
	}

	AccHairCount /= SampleCount;
	return AccHairCount;
}

void MainPS(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	const float2 UV = Input.UV;
	const uint2 PixelCoord = floor(Input.Position.xy);
	const float SceneDepth = ConvertFromDeviceZ(SceneDepthTexture.Load(uint3(PixelCoord, 0)).x);
	const bool bIsValidHairPixel = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelCoord, 0)).x > 0;
	const bool bIsHairPixel = bIsValidHairPixel && HairStrands.HairCoverageTexture.Load(uint3(PixelCoord, 0)) >= 1;

	float MinVisibility = 1.f;
	#if 1
	for (uint AtlasSlotIt=0;AtlasSlotIt<EffectiveAtlasSlotCount;++AtlasSlotIt)
	#else
	const uint AtlasSlotIt = 0;
	#endif
	{
		const uint AtlasSlotIndex = DeepShadow_AtlasSlotIndexBuffer[AtlasSlotIt];
		const float3 TranslatedWorldPosition = GetTranslatedWorldPosition(UV, SceneDepth);
		float4 AtlasScaleBias;
		float4 LightSpacePosition = ToLightPosition(TranslatedWorldPosition, AtlasSlotIndex, AtlasScaleBias);

		// PCF or simple point sampler
		float Visibility = 1;
		const bool bIsValid =
			LightSpacePosition.w > 0 &&
			(LightSpacePosition.x >= 0 && LightSpacePosition.x <= 1) &&
			(LightSpacePosition.y >= 0 && LightSpacePosition.y <= 1);
		if (!bIsHairPixel && bIsValid)
		{
			LightSpacePosition.xy = LightSpacePosition.xy * AtlasScaleBias.xy + AtlasScaleBias.zw;

			uint2 ShadowAtlasResolution;
			DeepShadow_FrontDepthTexture.GetDimensions(ShadowAtlasResolution.x, ShadowAtlasResolution.y);

		#if OPAQUE_PCF == 1
			FHairSamplingSettings ShadowSettings;
			ShadowSettings.ShadowDepthTexture = DeepShadow_FrontDepthTexture;
			ShadowSettings.ShadowDomTexture = DeepShadow_DomTexture;
			ShadowSettings.ShadowDepthTextureSampler = ShadowSampler;
			ShadowSettings.ShadowAtlasInvResolution = 1 / float2(DeepShadow_AtlasResolution);
			ShadowSettings.ShadowAtlasResolution = DeepShadow_AtlasResolution;
			ShadowSettings.SceneDepth = saturate(LightSpacePosition.z);
			#if PERMUTATION_KERNEL_TYPE == KERNEL_TYPE_4x4
				Visibility = 1 - InternalManual4x4PCF(LightSpacePosition.xy, ShadowSettings);
			#elif PERMUTATION_KERNEL_TYPE == KERNEL_TYPE_Gaussian8		
				Visibility = 1 - InternalGaussianFilter(LightSpacePosition.xy, ShadowSettings, 8);
			#elif PERMUTATION_KERNEL_TYPE == KERNEL_TYPE_Gaussian16
				Visibility = 1 - InternalGaussianFilter(LightSpacePosition.xy, ShadowSettings, 16);
			#elif PERMUTATION_KERNEL_TYPE == KERNEL_TYPE_GaussianTransmission8
				Visibility = 1 - InternalGaussianTransmissionFilter(LightSpacePosition.xyz, ShadowSettings, 8);
			#else//PERMUTATION_KERNEL_TYPE == KERNEL_TYPE_2x2
				Visibility = 1 - InternalManual2x2PCF(LightSpacePosition.xy, ShadowSettings);
			#endif	
		#else
			const float FrontDepth = DeepShadow_FrontDepthTexture.SampleLevel(LinearSampler, LightSpacePosition.xy, 0);
			const float ShadowBias = 0;
			Visibility = IsCloser(FrontDepth + ShadowBias, LightSpacePosition.z) ? 0 : 1;
		#endif

			MinVisibility = min(MinVisibility, Visibility);
		}
	}

	const float FadedShadow = lerp(1.0f, MinVisibility, FadeAlpha);
	OutColor = WriteShadowMaskOutput(FadedShadow, EncodingType);

}
#endif // SHADER_SHADOWMASK_DEEPSHADOW

////////////////////////////////////////////////////////////////////////////////////////////////
// Voxel volume

#if SHADER_SHADOWMASK_VOXEL

#if SHADER_SHADOWMASK_VOXEL
#define VOXEL_TRAVERSAL_DEBUG 0
#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#include "HairStrandsVoxelPageTraversal.ush"
#endif

#define USE_BLUE_NOISE 1
#include "../BlueNoise.ush"
DEFINE_HAIR_BLUE_NOISE_JITTER_FUNCTION

float4				Voxel_TranslatedLightPosition_LightDirection;
uint				Voxel_MacroGroupCount;
uint				Voxel_MacroGroupId;
uint				Voxel_RandomType;
Texture2D<float4>	RayMarchMaskTexture;
float 				FadeAlpha;
uint				EncodingType;

// The hair shadow mask is rendered by tracing ray/cone through the voxel structure. The traversing is done with 
// point sampling and jittering to get the smooth aspect, and avoid expensive trilinear sampling in a sparse 
// structure. These random functions make various tradeoff between randomness and correlation, so that the output 
// can coverge with TAA without any denoiser. They result from experimentation, and don't have a good grounding
float4 ComputeRandom4_0(uint2 PixelPosition, uint InSeed, uint JitterMode)
{
	InSeed = JitterMode > 1 ? 0 : InSeed;
	const float2 U0 = InterleavedGradientNoise(PixelPosition, InSeed);
	const float2 U1 = InterleavedGradientNoise(PixelPosition + 17, InSeed);
	return JitterMode > 0 ? float4(U0.x, U0.y, U1.x, U1.y) : float4(0,0,0,0);
}

float3 GetShadowMaskRandom(uint2 PixelPosition)
{
	float3 Random = 0.5f;
	if (Voxel_RandomType == 0)
	{
		Random = GetHairVoxelJitter(PixelPosition.xy, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);
	}
	else if (Voxel_RandomType == 1)
	{
		Random = ComputeRandom4_0(PixelPosition, View.StateFrameIndexMod8, VirtualVoxel.JitterMode).xyz;
	}
	else if (Voxel_RandomType == 2)
	{
		#if USE_BLUE_NOISE
		Random = GetHairBlueNoiseJitter(PixelPosition, 0 /*SampleIndex*/, 1 /*MaxSampleCount*/, View.StateFrameIndexMod8/*TimeIndex*/).xyz;
		#else
		Random = GetHairVoxelJitter2(PixelPosition, View.StateFrameIndexMod8, VirtualVoxel.JitterMode).xyz;
		#endif
	}
	return Random;
}

void MainPS(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0)
{
	OutColor = 1.f;

	const float2 UV = Input.UV;
	const uint2 PixelCoord = floor(Input.Position.xy);
	const float SceneDepth = ConvertFromDeviceZ(SceneDepthTexture.Load(uint3(PixelCoord, 0)).x);

	const float DistanceThreshold = 100000.0f;
	const float CoverageThreshold = 0.995f; // When Coverage is high, we do not trace shadow on opaque since hair/fur is covering the background.
	const bool bIsHairPixel = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelCoord, 0)).x > 0;
	const float HairPixelCoverage = HairStrands.HairCoverageTexture.Load(uint3(PixelCoord, 0));
	const bool bIsValidOpaquePixel = !bIsHairPixel || (bIsHairPixel && HairPixelCoverage < CoverageThreshold);
	
	float3 TranslatedWorldPosition = GetTranslatedWorldPosition(UV, SceneDepth);
	if (View.StereoPassIndex == 1)
	{
		TranslatedWorldPosition += VirtualVoxel.TranslatedWorldOffsetStereoCorrection;
	}

	// PCF or simple point sampler
	float Visibility = 1;
	bool bHasValidOutput = false;
	bool bWriteResult = false;
	if (bIsValidOpaquePixel)
	{
		const float3 TranslatedLightPosition= GetTranslatedLightPosition(Voxel_TranslatedLightPosition_LightDirection, TranslatedWorldPosition);
		const float3 LightDirection			= GetTranslatedLightDirection(Voxel_TranslatedLightPosition_LightDirection, TranslatedWorldPosition);

		// Depth bias
		// Origin is shifted 2 voxels away towards the light + a constant bias of the size of the voxel
		const float3 Random = GetShadowMaskRandom(PixelCoord.xy);
		float3 NormalizedDepthBias = 0.f;
		{
			const float PositionBiasScale = 0.5f;
			NormalizedDepthBias = (VirtualVoxel.DepthBiasScale_Shadow * LightDirection + PositionBiasScale * (Random * 2 - 1));
		}

		FVirtualVoxelCommonDesc CommonDesc;
		CommonDesc.PageCountResolution	= VirtualVoxel.PageCountResolution;
		CommonDesc.PageTextureResolution= VirtualVoxel.PageTextureResolution;
		CommonDesc.PageResolution		= VirtualVoxel.PageResolution;
		CommonDesc.PageResolutionLog2	= VirtualVoxel.PageResolutionLog2;

		#if VOXEL_TRAVERSAL_DEBUG
		const bool bDebugEnabled = PixelCoord.x == GetCursorPos().x && PixelCoord.y == GetCursorPos().y;
		#else
		const bool bDebugEnabled = false;
		#endif	

		const bool bHasPixelVisibleLight = RayMarchMaskTexture.Load(uint3(PixelCoord, 0)).r > 0.0f;
		if (bHasPixelVisibleLight)
		{
			#if PERMUTATION_USE_ONEPASS
			for (uint GroupIt=0;GroupIt< Voxel_MacroGroupCount;++GroupIt)
			#else
			const uint GroupIt = Voxel_MacroGroupId;
			#endif
			{

				const FPackedVirtualVoxelNodeDesc PackedNode = VirtualVoxel.NodeDescBuffer[GroupIt];
				const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(PackedNode, VirtualVoxel.PageResolution);
				const bool bIsValid = all(NodeDesc.PageIndexResolution != 0);
				if (bIsValid)
				{
					FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
					TraversalSettings.DensityScale = VirtualVoxel.DensityScale_Shadow;
					TraversalSettings.CountThreshold = 1;
					TraversalSettings.DistanceThreshold = DistanceThreshold;
					TraversalSettings.bDebugEnabled = bDebugEnabled;
					TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Shadow;
					TraversalSettings.Random = Random;
					TraversalSettings.PixelRadius = ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, SceneDepth);
					TraversalSettings.bCastShadow = true;

					FHairTraversalResult Result = ComputeHairCountVirtualVoxel(
						TranslatedWorldPosition + NormalizedDepthBias * NodeDesc.VoxelWorldSize,
						TranslatedLightPosition,
						CommonDesc,
						NodeDesc,
						VirtualVoxel.PageIndexBuffer,
						VirtualVoxel.PageTexture,
						TraversalSettings);
					Visibility = min(Visibility, saturate(1 - Result.HairCount));

					bWriteResult = bWriteResult || Result.HairCount > 0;
					bHasValidOutput = true;
				}
			}
		}
	}

	if (!bHasValidOutput)
	{
		discard;
	}

	if (bWriteResult)
	{
		const float FadedShadow = lerp(1.0f, Visibility, FadeAlpha);
		OutColor = WriteShadowMaskOutput(FadedShadow, EncodingType);
	}
}

#endif // SHADER_SHADOWMASK_VOXEL