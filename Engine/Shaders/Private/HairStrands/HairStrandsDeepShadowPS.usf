// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "HairStrandsDeepShadowCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVoxelPageCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

static const float FixPointMaxValue = 1000;	// 100m @hair_todo: make this camera relative, and expose a CVAR fix changing this value 
static const float FixPointScale	= 100;	// 0.1mm precision
static const float FixPointRange	= 2 * FixPointMaxValue * FixPointScale;    

#define SUPPORT_OPACITY_MASK 0

void MainDepth(
	in float4 SvPosition	: SV_Position)
{
#if SUPPORT_OPACITY_MASK
	ResolvedView = ResolveView();
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
	clip(GetMaterialMask(PixelMaterialInputs));
#endif
}

void MainDom(
	in float4 SvPosition	: SV_Position,
	in float HairCoverage	: CUSTOM_COVERAGE,
	out float4 OutColor		: SV_Target0)
{
	#if SUPPORT_OPACITY_MASK
	{
		ResolvedView = ResolveView();
		FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
		FPixelMaterialInputs PixelMaterialInputs;
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
		clip(GetMaterialMask(PixelMaterialInputs));
	}
	#endif

	const uint2 PixelCoord = uint2(floor(SvPosition.xy));
	const float FrontDepth = DeepRasterPass.FrontDepthTexture.Load(uint3(PixelCoord, 0));
	const float DistanceToFrontDepth = GetDomDistanceToFrontDepth(FrontDepth, SvPosition.z);

	OutColor = ComputeDOMWeights(DistanceToFrontDepth, DeepRasterPass.LayerDepths) * HairCoverage;
}