// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "HairStrandsVisibilityCommon.ush"

#define MESH_RENDER_DEPTH 0
#define MESH_RENDER_DOM   1

#if MESH_RENDER_MODE != MESH_RENDER_DEPTH && MESH_RENDER_MODE != MESH_RENDER_DOM
#error Unknown DeepShadow render mode 
#endif

struct FDeepShadowAccumulateVSToPS
{
#if SUPPORT_TANGENT_PROPERTY == 1 || SUPPORT_MATERIAL_PROPERTY == 1
	FVertexFactoryInterpolantsVSToPS Interpolants;
#endif
	float4 Position : SV_POSITION;
#if MESH_RENDER_MODE == MESH_RENDER_DOM
	float HairCoverage : CUSTOM_COVERAGE;
#endif
};

#define VS_OUTPUT_TYPE FDeepShadowAccumulateVSToPS
#define SUPPORT_WPO 0

#if VERTEXSHADER 

/** transform mesh as normal */ 
void Main(
	FVertexFactoryInput Input,
	out VS_OUTPUT_TYPE Output
)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);

#ifdef HAIR_STRAND_MESH_FACTORY
	const FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits);
	const FDeepShadowViewInfo DeepShadowViewInfo = DeepRasterPass.DeepShadowViewInfoBuffer[DeepRasterPass.AtlasSlotIndex];

	FHairViewInfo HairViewInfo;
	HairViewInfo.TranslatedWorldCameraOrigin = ResolvedView.TranslatedWorldCameraOrigin;
	HairViewInfo.ViewForward = DeepShadowViewInfo.ViewForward;
	HairViewInfo.RadiusAtDepth1 = DeepShadowViewInfo.MinRadiusAtDepth1;
	HairViewInfo.bIsOrthoView = HairRenderInfo.bIsOrthoView;

	const float4x4 TranslatedWorldToClipMatrix = DeepShadowViewInfo.TranslatedWorldToClipScaledBiased;
	const float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates, HairViewInfo);
#else
	const float4x4 TranslatedWorldToClipMatrix = 0;
	const float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
#endif

	Output.Position = mul(TranslatedWorldPosition, TranslatedWorldToClipMatrix);

#ifdef HAIR_STRAND_MESH_FACTORY	
	const float CurrentDepth = Output.Position.z / Output.Position.w;
	const float PixelRadius = HairRenderInfo.bIsOrthoView ? HairRenderInfo.RadiusAtDepth1Primary : (CurrentDepth * HairRenderInfo.RadiusAtDepth1Primary);
	const float StrandRealRadius = VFIntermediates.HairDimensions.y;
	const float Coverage = StrandRealRadius / max(StrandRealRadius, PixelRadius);
	const float HairCoverage = Coverage * VFIntermediates.HairDensity;
#else
	const float HairCoverage = 1;
#endif

#if MESH_RENDER_MODE == MESH_RENDER_DOM
	Output.HairCoverage = HairCoverage;
#endif
}

#endif // VERTEXSHADER
