// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FHairTransmittanceMask
{
	float HairCount;
	float Visibility;
};

FHairTransmittanceMask InitHairTransmittanceMask()
{
	FHairTransmittanceMask Out;
	Out.HairCount = 0;
	Out.Visibility = 1;
	return Out;
}

FHairTransmittanceMask InitHairTransmittanceMask(float HairCount, float Visibility)
{
	FHairTransmittanceMask Out;
	Out.HairCount = HairCount;
	Out.Visibility = Visibility;
	return Out;
}

uint PackTransmittanceMask(FHairTransmittanceMask In)
{
	return min(uint(In.HairCount * 1000), uint(0x00FFFFFF)) | (min(uint(In.Visibility * 0xFF), uint(0xFF)) << 24);
}

uint InitNullPackedHairTransmittanceMask()
{
	return 0;
}

FHairTransmittanceMask UnpackTransmittanceMask(uint In)
{
	FHairTransmittanceMask Out;
	Out.HairCount = float(In & 0x00FFFFFF) / 1000.f;
	Out.Visibility = float((In & 0xFF000000) >> 24) / 255.f;
	return Out;
}

bool IsLightDirectional(float4 InTranslatedLightPosition_LightDirection)
{
	return InTranslatedLightPosition_LightDirection.w == 0;
}

// Return the direction from the world position to the light
float3 GetTranslatedLightDirection(float4 InTranslatedLightPosition_LightDirection, float3 InTranslatedWorldPosition)
{
	const bool bIsDirectional = InTranslatedLightPosition_LightDirection.w == 0;
	return bIsDirectional ? -InTranslatedLightPosition_LightDirection.xyz : normalize(InTranslatedLightPosition_LightDirection.xyz - InTranslatedWorldPosition);
}

float3 GetTranslatedLightPosition(float4 InTranslatedLightPosition_LightDirection, float3 InTranslatedWorldPosition)
{
	const float VoxelShadowMaxDistance = 100000.0; // 1 km shadow distance	
	const float3 TranslatedLightDirection = GetTranslatedLightDirection(InTranslatedLightPosition_LightDirection, InTranslatedWorldPosition);

	const bool bIsDirectional = InTranslatedLightPosition_LightDirection.w == 0;
	return bIsDirectional ? InTranslatedWorldPosition + TranslatedLightDirection * VoxelShadowMaxDistance : InTranslatedLightPosition_LightDirection.xyz;
}
