// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef HAIR_COMPONENT_LS
	#error HAIR_COMPONENT_LS is not defined, please include ShadingCommon.ush
#endif

#ifndef HAIR_COMPONENT_GS
	#error HAIR_COMPONENT_GS is not defined, please include ShadingCommon.ush
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define HairScatteringLUTSampler View.SharedBilinearClampedSampler
#else
#define HairScatteringLUTSampler View.HairScatteringLUTSampler
#endif

float Hair_g2(float Variance,float Theta)
{
	//const float A = 1.f / sqrt(2 * PI * Variance);
	const float A = 1.f;
	return A * exp(-0.5 * Pow2(Theta) / Variance);
}

// Dual scattering computation are done here for faster iteration (i.e., does not invalidate tons of shaders)
FHairTransmittanceData ComputeDualScatteringTerms(
	const FHairTransmittanceMask TransmittanceMask,
	const FHairAverageScattering AverageScattering,
	float Roughness,
	const float Backlit,
	const float3 V,
	const float3 L,
	const float3 T,
	const uint HairComponents)
{
	const float SinThetaL = clamp(dot(T, L), -1, 1);
	const float SinThetaV = clamp(dot(T, V), -1, 1);
	const float CosThetaL = sqrt(1 - SinThetaL * SinThetaL);
	const float MaxAverageScatteringValue = 0.99f;

	// Straight implementation of the dual scattering paper 
	const float3 af	 = min(MaxAverageScatteringValue.xxx, AverageScattering.A_front);
	const float3 af2 = Pow2(af);
	const float3 ab  = min(MaxAverageScatteringValue.xxx, AverageScattering.A_back);
	const float3 ab2 = Pow2(ab);
	const float3 OneMinusAf2 = 1 - af2;

	const float3 A1 = ab * af2 / OneMinusAf2;
	const float3 A3 = ab * ab2 * af2 / (OneMinusAf2*Pow2(OneMinusAf2));
	const float3 Ab = A1 + A3;

	// Add a min/max roughness for dual scattering based. This is a bit adhoc, but 
	// * Min/lower bound helps with BSDF being too narrow and causing some fireflies, 
	// * Max/upper bound helps against "too-flat" look due to dual scattering assuming directional lobe (vs. more radially uniform)
	Roughness = clamp(Roughness, 0.18f, 0.6f);
	const float Beta_R	 = Pow2( Roughness );
	const float Beta_TT	 = Pow2( Roughness / 2 ); 
	const float Beta_TRT = Pow2( Roughness * 2 ); 

	const float Shift     = 0.035;
	const float Shift_R   =-0.035*2;
	const float Shift_TT  = 0.035;
	const float Shift_TRT = 0.035*4;

	// Average density factor (This is the constant used in the original paper)
	const float df = 0.7f;
	const float db = 0.7f;

	// Always shift the hair count by one to remove self-occlusion/shadow aliasing and have smoother transition
	// This insure the the pow function always starts at 0 for front facing hair
	const float HairCount = max(0, TransmittanceMask.HairCount - 1);

	// This is a coarse approximation of eq. 13. Normally, Beta_f should be weighted by the 'normalized' 
	// R, TT, and TRT terms 
	const float3 af_weights = af / (af.r + af.g + af.b);
	const float3 Beta_f  = dot(float3(Beta_R, Beta_TT, Beta_TRT), af_weights);
	const float3 Beta_f2 = Beta_f*Beta_f;
	const float3 sigma_f2 = Beta_f2 * max(1.f, HairCount);

	const float Theta_d = asin(SinThetaL) + asin(SinThetaV);
	const float Theta_h = Theta_d * 0.5f;

	// Global scattering spread 'Sf'
	float3 Sf = float3(	Hair_g2(sigma_f2.r, Theta_h), 
						Hair_g2(sigma_f2.g, Theta_h), 
						Hair_g2(sigma_f2.b, Theta_h)) / PI;
	const float3 Tf = pow(AverageScattering.A_front, HairCount);

	// Overall shift due to the various local scatteing event (all shift & roughnesss should vary with color)
	const float3 shift_f = dot(float3(Shift_R, Shift_TT, Shift_TRT), af_weights);
	const float3 shift_b = shift_f;
	const float3 delta_b = shift_b * (1 - 2*ab2 / Pow2(1 - af2)) * shift_f * (2 * Pow2(1 - af2) + 4*af2*ab2)/Pow3(1-af2);

	const float3 ab_weights = ab / (ab.r + ab.g + ab.b);
	const float3 Beta_b  = dot(float3(Beta_R, Beta_TT, Beta_TRT), ab_weights);
	const float3 Beta_b2 = Beta_b * Beta_b;

	const float3 sigma_b = (1 + db*af2) * (ab*sqrt(2*Beta_f2 + Beta_b2) + ab*ab2*sqrt(2*Beta_f2 + Beta_b2)) / (ab + ab*ab2*(2*Beta_f + 3*Beta_b));
	const float3 sigma_b2 = sigma_b * sigma_b;

	// Local scattering Spread 'Sb'
	// In Efficient Implementation of the Dual Scattering Model, the variance for back scattering is the sum of the front & back variances
	float3 Sb = float3(	Hair_g2(sigma_f2.r + sigma_b2.r, Theta_h - delta_b.r),
						Hair_g2(sigma_f2.g + sigma_b2.g, Theta_h - delta_b.g),
						Hair_g2(sigma_f2.b + sigma_b2.b, Theta_h - delta_b.b)) / PI;


	// Different variant for managing sefl-occlusion issue for global scattering
	const float3 GlobalScattering = lerp(1, Tf * Sf * df, saturate(HairCount));
	const float3 LocalScattering  = 2 * Ab * Sb * db;
	
	// Decode if the Local/Global component should be enabled or not (for debug purpose)
	FHairTransmittanceData Out = InitHairStrandsTransmittanceData();
	Out.ScatteringComponent = HairComponents | HAIR_COMPONENT_MULTISCATTER;
	Out.GlobalScattering = (HairComponents & HAIR_COMPONENT_GS) > 0 ? GlobalScattering : 1;
	Out.LocalScattering  = (HairComponents & HAIR_COMPONENT_LS) > 0 ? LocalScattering : 0;
	Out.bUseLegacyAbsorption = (HairComponents & HAIR_COMPONENT_TT_MODEL) == 0;
	Out.OpaqueVisibility = TransmittanceMask.Visibility;
	return Out;
	// Final computation is done in ShadingModels.ush with the following formula
	// GlobalScattering * (Fs + LocalScattering) * TransmittanceMask.OpaqueVisibility;
}
	
FHairTransmittanceData GetHairTransmittance(
	const float3 V,
	const float3 L,
	FGBufferData GBuffer,
	FHairTransmittanceMask TransmittanceMask,
	Texture3D<float4> HairLUTTexture,
	SamplerState HairLUTSampler,
	const uint HairComponents)
{
	// Hack: Override the actual roughness, with a different value to achieve a specific look
	const float Roughness = GBuffer.CustomData.x > 0 ? GBuffer.CustomData.x : GBuffer.Roughness;
	const float Backlit = GBuffer.CustomData.z;

	// Compute the transmittance based on precompute Hair transmittance LUT
	const float3 T = GBuffer.WorldNormal;
	const float SinLightAngle = dot(L, T);
	const FHairAverageScattering AverageScattering = SampleHairLUT(HairLUTTexture, HairLUTSampler, GBuffer.BaseColor, Roughness, SinLightAngle);

	FHairTransmittanceData Out = ComputeDualScatteringTerms(
		TransmittanceMask,
		AverageScattering,
		Roughness,
		Backlit,
		V,
		L,
		T,
		HairComponents);

	return Out;
}

float3 GetHairTransmittanceOnly(FHairTransmittanceMask TransmittanceMask, FGBufferData GBuffer, float SinLightAngle, Texture3D<float4> InHairScatteringLUTTexture, SamplerState InHairLUTSampler)
{
	// Always shift the hair count by one to remove self-occlusion/shadow aliasing and have smoother transition
	// This insure the the pow function always starts at 0 for front facing hair
	const float HairCount = max(0.f, TransmittanceMask.HairCount - 1.f);

	const FHairAverageScattering AbsorptionData = SampleHairLUT(InHairScatteringLUTTexture, InHairLUTSampler, GBuffer.BaseColor, GBuffer.Roughness, SinLightAngle);
	return pow(AbsorptionData.A_front, HairCount) * TransmittanceMask.Visibility;
}

// Evaluate local scattering scattering
FHairTransmittanceData EvaluateDualScattering(inout float3 BaseColor, float3 WorldNormal, float Roughness, float3 V, float3 L)
{
	// Prevent numerical issue causing NaN during hair lighting (i.e., during baseColor/luma(baseColor) computation)
	BaseColor = max(0.0001f, BaseColor);

	// Compute the transmittance based on precompute Hair transmittance LUT
	const float SinLightAngle = dot(L, WorldNormal);
	const FHairAverageScattering AverageScattering = SampleHairLUT(View.HairScatteringLUTTexture, HairScatteringLUTSampler, BaseColor, Roughness, SinLightAngle);

	return ComputeDualScatteringTerms(
		InitHairTransmittanceMask(),
		AverageScattering,
		Roughness,
		1, // Backlit / GBuffer.CustomData.z;
		V,
		L,
		WorldNormal,
		View.HairComponents);
}

FHairTransmittanceData EvaluateDualScattering(inout FGBufferData GBuffer, float3 V, float3 L)
{
	checkSlow(GBuffer.ShadingModelID == SHADINGMODELID_HAIR);
	return EvaluateDualScattering(GBuffer.BaseColor, GBuffer.WorldNormal, GBuffer.Roughness, V, L);
}
