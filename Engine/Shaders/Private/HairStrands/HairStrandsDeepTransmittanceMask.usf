// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTextureParameters.ush"
#include "../DeferredShadingCommon.ush"
#include "../ShadowFilteringCommon.ush"
#include "HairStrandsDeepShadowCommon.ush"
#include "HairStrandsCommon.ush"
#include "HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVoxelPageCommon.ush"

#define DEBUG_ENABLE 0
#if DEBUG_ENABLE
	#include "../ShaderPrint.ush"
#endif //DEBUG_ENABLE

#ifndef MAX_HAIR_MACROGROUP_COUNT
#error MAX_HAIR_MACROGROUP_COUNT needs to be defined
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////
// Utils function & structs
struct FTransmittanceSettings
{
	int2	 AtlasResolution;
	float4x4 TranslatedWorldToLightTransform;
	float4   AtlasScaleBias;
	float3   LightDirection; // Direction from world position to the light
	float3   TranslatedLightPosition;
	bool	 bIsDirectional;

	float    DeepShadowDepthBiasScale;
	float    DeepShadowDensityScale;
	float	 TransmittanceKernelApertureInDegree;
	float3   Random;
	uint	 DebugMode;

	uint TransmittanceKernelType;
};

float3 GetViewVector(float3 InTranslatedWorldPosition)
{
	float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(InTranslatedWorldPosition);
	float3 V = -CameraToPixel;
	return V;
}

float3 GetTranslatedWorldPosition(float2 UV, float SceneDepth)
{
	const float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	return mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
}

////////////////////////////////////////////////////////////////////////////////////////////////
// Defines

// Need to match FDeepShadowData::MaxMacroGroupCount in HairRendering.h
#if SHADER_TRANSMITTANCE_DEEPSHADOW || SHADER_TRANSMITTANCE_VOXEL 
	#if PERMUTATION_GROUP_SIZE == 64
		#define TRANSMITTANCE_GROUP_SIZE 64
	#elif PERMUTATION_GROUP_SIZE == 32
		#define TRANSMITTANCE_GROUP_SIZE 32
	#else
		#error Unknown group size	
	#endif
#endif

////////////////////////////////////////////////////////////////////////////////////////////////
// Deep shadow

#if SHADER_TRANSMITTANCE_DEEPSHADOW

StructuredBuffer<uint> DeepShadow_AtlasSlotIndexBuffer;
StructuredBuffer<FDeepShadowViewInfo> DeepShadow_ViewInfoBuffer;

uint2				DeepShadow_AtlasResolution;
uint				LightChannelMask;
float4				ShadowChannelMask;
float4				TranslatedLightPosition_LightDirection;
float				DeepShadow_DepthBiasScale;
float				DeepShadow_DensityScale;
float				LightRadius;
float				DeepShadow_KernelAperture;
float4				DeepShadow_LayerDepths;
uint				DeepShadow_KernelType;
uint				DeepShadow_DebugMode;
Texture2D<float>	DeepShadow_FrontDepthTexture;
Texture2D<float4>	DeepShadow_DomTexture;

Buffer<uint4>		IndirectArgsBuffer;
RWBuffer<uint>		OutTransmittanceMask;

FHairTransmittanceMask ComputeTransmittance(
	float3 TranslatedWorldPosition,
	FTransmittanceSettings Settings,
	float4				InLayerDepths,
	Texture2D<float>	FrontDepthTexture,
	Texture2D<float4>	DomTexture)
{
	FHairTransmittanceMask Out = InitHairTransmittanceMask();
	{
		// LightSpacePosition is 'TranslatedWorldPosition' in LightSpace (TexelCoord.XY / NDC Depth.Z)
		const float3 LightSpacePosition = ToLightPosition(TranslatedWorldPosition, Settings.TranslatedWorldToLightTransform) * float3(Settings.AtlasScaleBias.xy, 1) + float3(Settings.AtlasScaleBias.zw, 0);

		float DepthBias = 0;
		if (Settings.DeepShadowDepthBiasScale > 0)
		{
			#if 0 
			const float DephtBiasSlope = 6;
			const float DefaultDepthBias = LAYER_DEPTH0 * Settings.DeepShadowDepthBiasScale; 
			DepthBias = (CosLightAngle*DephtBiasSlope + 1) * DefaultDepthBias;
			#else
			DepthBias = InLayerDepths[0] * Settings.DeepShadowDepthBiasScale;
			#endif
		}

		// Compute the number of hair count between light & shading point
		if (Settings.TransmittanceKernelType == 4)
		{
			//Out = SampleDOM_PCF_Accurate(LightSpacePosition.xyz, DepthBias, InLayerDepths, FrontDepthTexture, DomTexture, GBuffer, SinLightAngle, HairLUTTexture);
			Out.HairCount = 0;
			Out.Visibility = 1;
		}
		else
		{
			float HairCount = 0;
			if (Settings.TransmittanceKernelType == 3)
			{
				HairCount = SampleDOM_PCSS(LightSpacePosition.xyz, Settings.AtlasResolution, DepthBias, InLayerDepths, FrontDepthTexture, DomTexture, Settings.TransmittanceKernelApertureInDegree, Settings.Random.xy);
			}
			else if (Settings.TransmittanceKernelType == 2)
			{
				HairCount = SampleDOM_PCF(LightSpacePosition.xyz, DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
			}	
			else if (Settings.TransmittanceKernelType == 1)
			{
				HairCount = SampleDOM_PCF2x2(LightSpacePosition.xyz, DepthBias, InLayerDepths, FrontDepthTexture, DomTexture);
			}
			else
			{
				HairCount = 0;
			}
			Out.HairCount = HairCount * Settings.DeepShadowDensityScale;
			Out.Visibility = 1;
		}

		if (Settings.DebugMode != 0)
		{
			Out.HairCount = 1;
		}
	}
	return Out;
}

[numthreads(TRANSMITTANCE_GROUP_SIZE, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	// Note: Use a fixed group count width (HAIR_VISIBILITY_GROUP_COUNT_WIDTH) for avoiding loading the indirect args (to avoid dep. memory fetch)
	const uint SampleIndex = DispatchThreadId.x + DispatchThreadId.y * TRANSMITTANCE_GROUP_SIZE * HAIR_VISIBILITY_GROUP_COUNT_WIDTH;

	const uint MaxVisibilityNodeCount = HairStrands.HairSampleCount[0];
	if (SampleIndex >= MaxVisibilityNodeCount)
	{
		return;
	}

	const uint2 PixelCoord = HairStrands.HairSampleCoords[SampleIndex];

	const float2 UV = (PixelCoord + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	const float3 TransmittanceRandom = float3(InterleavedGradientNoise(PixelCoord, 1), InterleavedGradientNoise(PixelCoord, 2), InterleavedGradientNoise(PixelCoord, 3));

	{
		const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[SampleIndex]);

		if ((LightChannelMask & Sample.LightChannelMask) == 0)
		{
			OutTransmittanceMask[SampleIndex] = InitNullPackedHairTransmittanceMask();
			return;
		}

		const uint MacroGroupIndex = clamp(Sample.MacroGroupId, 0, MAX_HAIR_MACROGROUP_COUNT - 1);
		const uint AtlasSlotIndex = DeepShadow_AtlasSlotIndexBuffer[MacroGroupIndex];

		const FDeepShadowViewInfo ShadowViewInfo = DeepShadow_ViewInfoBuffer[AtlasSlotIndex];
		const float4x4 TranslatedWorldToLightTransform = ShadowViewInfo.TranslatedWorldToClip;
		const float4 AtlasScaleBias = ShadowViewInfo.AtlasScaleBias * float4(DeepShadow_AtlasResolution, DeepShadow_AtlasResolution);
		
		const float SceneDepth = ConvertFromDeviceZ(Sample.Depth);
		const float3 TranslatedWorldPosition = GetTranslatedWorldPosition(UV, SceneDepth);

		FTransmittanceSettings Settings;
		Settings.AtlasResolution = DeepShadow_AtlasResolution;
		Settings.TranslatedWorldToLightTransform = TranslatedWorldToLightTransform;
		Settings.AtlasScaleBias = AtlasScaleBias;
		Settings.bIsDirectional = IsLightDirectional(TranslatedLightPosition_LightDirection);
		Settings.LightDirection = GetTranslatedLightDirection(TranslatedLightPosition_LightDirection, TranslatedWorldPosition);
		Settings.TranslatedLightPosition = GetTranslatedLightPosition(TranslatedLightPosition_LightDirection, TranslatedWorldPosition);
		Settings.DeepShadowDepthBiasScale = DeepShadow_DepthBiasScale;
		Settings.DeepShadowDensityScale = DeepShadow_DensityScale;
		Settings.TransmittanceKernelApertureInDegree = DeepShadow_KernelAperture;
		Settings.TransmittanceKernelType = DeepShadow_KernelType;
		Settings.Random = TransmittanceRandom;
		Settings.DebugMode = DeepShadow_DebugMode;

		const FHairTransmittanceMask TransmittanceMask = ComputeTransmittance(TranslatedWorldPosition, Settings, DeepShadow_LayerDepths, DeepShadow_FrontDepthTexture, DeepShadow_DomTexture);

		OutTransmittanceMask[SampleIndex] = PackTransmittanceMask(TransmittanceMask);
	}
}
#endif // #if SHADER_TRANSMITTANCE_DEEPSHADOW

////////////////////////////////////////////////////////////////////////////////////////////////
// Voxel volume

#if SHADER_TRANSMITTANCE_VOXEL

#if PERMUTATION_TRAVERSAL == 1
	#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#else
	#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR
#endif

#define VOXEL_TRAVERSAL_DEBUG 0
#include "HairStrandsVoxelPageTraversal.ush"
#include "../LightGridCommon.ush"

#if VOXEL_TRAVERSAL_DEBUG
#include "../ShaderPrint.ush"
void PrintInfo(uint Index, uint2 PixelCoord, FHairTransmittanceMask TransmittanceMask)
{
	#if 0
	const uint2 OutputResolution = uint2(1024, 1024); 
	float2 Origin = (PixelCoord) / float2(OutputResolution);
	Origin.y += (Index % 8) * 2 * ShaderPrintUniform.FontSize.w;

	FShaderPrintContext Context = InitShaderPrintContext(true, Origin);

	Print(Context, TEXT("Hair Count "));
	Print(Context, TransmittanceMask.HairCount);
	Newline(Context);

	Print(Context, TEXT("Visibility "));
	Print(Context, TransmittanceMask.Visibility);
	Newline(Context);

	#endif
}
#endif // VOXEL_TRAVERSAL_DEBUG

uint				LightChannelMask;
float4				TranslatedLightPosition_LightDirection;
float				LightRadius;
#if PERMUTATION_ONE_PASS
Texture2D<uint4> 	ShadowMaskBitsTexture;
#else
Texture2D<float4> 	RayMarchMaskTexture;
float4				ShadowChannelMask;
#endif
Buffer<uint>		IndirectArgsBuffer;
RWBuffer<uint>		OutTransmittanceMask;

#include "../VirtualShadowMaps/VirtualShadowMapLightGrid.ush"

#define USE_BLUE_NOISE 1
#include "../BlueNoise.ush"
DEFINE_HAIR_BLUE_NOISE_JITTER_FUNCTION
#include "HairStrandsScreenTracing.ush"
uint bUseScreenTrace;

float3 InternalGetHairVoxelJitter(uint2 PixelCoord, uint SampleIndex, uint MaxSampleCount, uint TimeIndex, uint JitterMode)
{
	// Blue noise is cheaper to compute and h
	#if USE_BLUE_NOISE
	float3 RandomSample = GetHairBlueNoiseJitter(PixelCoord, SampleIndex, MaxSampleCount, JitterMode > 1 ? 0 : TimeIndex).xyz;
	#else
	float3 RandomSample = GetHairVoxelJitter(PixelCoord, View.StateFrameIndexMod8, JitterMode);
	#endif

	return JitterMode > 0 ? RandomSample : float3(0,0,0);
}

FHairTransmittanceMask ComputeTransmittanceVoxel(
	const uint2 PixelCoord,
	const float PixelRadius,
	const float SceneDepth,
	float3 TranslatedWorldPosition,
	uint MacroGroupId,
	FTransmittanceSettings Settings,
	bool bDebugEnabled)
{
	// Cone opening - Area light transmission is disabled as it is too noisy for now
	const float TanLightAngle = 0; // Settings.bIsDirectional ? 0 : LightRadius / length(Settings.TranslatedLightPosition - TranslatedWorldPosition);

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[MacroGroupId], VirtualVoxel.PageResolution);

	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution	= VirtualVoxel.PageCountResolution;
	CommonDesc.PageTextureResolution= VirtualVoxel.PageTextureResolution;
	CommonDesc.PageResolution		= VirtualVoxel.PageResolution;
	CommonDesc.PageResolutionLog2	= VirtualVoxel.PageResolutionLog2;

	// Compute the number of hair count between light & shading point
	const float DistanceThreshold = 100000;
	float AvgHairCount = 0;
	float AvgHairVisility = 0;
	#if PERMUTATION_SUPERSAMPLING
	const uint SampleCount = 8; 
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	#else
	const uint SampleCount = 1; 
	const uint SampleIt = 0;
	#endif
	{
		float3 SampleRandom = InternalGetHairVoxelJitter(PixelCoord, SampleIt, SampleCount, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);
	
		const float PositionBiasScale = 0.5f;
		const float3 DepthBias = NodeDesc.VoxelWorldSize * (VirtualVoxel.DepthBiasScale_Transmittance * Settings.LightDirection + PositionBiasScale*(SampleRandom*2-1));
		const float3 SampleTranslatedWorldPosition = TranslatedWorldPosition + DepthBias;

		FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
		TraversalSettings.DensityScale = VirtualVoxel.DensityScale_Transmittance;
		TraversalSettings.CountThreshold = GetOpaqueVoxelValue();
		TraversalSettings.DistanceThreshold = DistanceThreshold;
		TraversalSettings.bDebugEnabled = bDebugEnabled;
		TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Transmittance;
		TraversalSettings.Random = SampleRandom;
		TraversalSettings.TanConeAngle = TanLightAngle;
		TraversalSettings.PixelRadius = PixelRadius;

		const FHairTraversalResult Result = ComputeHairCountVirtualVoxel(
			SampleTranslatedWorldPosition,
			Settings.TranslatedLightPosition,
			CommonDesc,
			NodeDesc,
			VirtualVoxel.PageIndexBuffer,
			VirtualVoxel.PageTexture,
			TraversalSettings);

		if (bUseScreenTrace)
		{
			AvgHairCount += ComputeScreenTraceHairCount(PixelCoord, PixelRadius, SceneDepth, TranslatedWorldPosition, Settings.LightDirection, SampleIt, SampleCount, VirtualVoxel.JitterMode > 0);
		}

		AvgHairCount += Result.HairCount;
		AvgHairVisility += Result.Visibility;
	}

	#if PERMUTATION_SUPERSAMPLING
	AvgHairCount /= max(1u, SampleCount);
	AvgHairVisility /= max(1u, SampleCount);
	#endif

	FHairTransmittanceMask Out;
	Out.HairCount = AvgHairCount;
	Out.Visibility = AvgHairVisility;
	return Out;
}

[numthreads(TRANSMITTANCE_GROUP_SIZE, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
#if PERMUTATION_ONE_PASS
{
	const uint SampleIndex = DispatchThreadId.x + DispatchThreadId.y * TRANSMITTANCE_GROUP_SIZE * HAIR_VISIBILITY_GROUP_COUNT_WIDTH;

	const uint MaxVisibilityNodeCount = HairStrands.HairSampleCount[0];
	if (SampleIndex >= MaxVisibilityNodeCount)
	{
		return;
	}
	const FHairSample Sample	= UnpackHairSample(HairStrands.HairSampleData[SampleIndex]);
	const uint MacroGroupIndex	= clamp(Sample.MacroGroupId, 0, MAX_HAIR_MACROGROUP_COUNT - 1);
	const float SceneDepth		= ConvertFromDeviceZ(Sample.Depth);
	const uint2 PixelCoord		= HairStrands.HairSampleCoords[SampleIndex];
	const float2 UV				= (PixelCoord.xy + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	const float PixelRadius		= ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, SceneDepth);
	float3 TranslatedWorldPosition = GetTranslatedWorldPosition(UV, SceneDepth);
	if (View.StereoPassIndex == 1)
	{
		TranslatedWorldPosition += VirtualVoxel.TranslatedWorldOffsetStereoCorrection;
	}

	const uint4 PackedShadowMask = ShadowMaskBitsTexture[PixelCoord];
	const float3 Random = GetHairVoxelJitter(PixelCoord.xy, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);


	// Use VSM light grid index remapping to ensure we compute transmittance value only for shadowed light
	// The logic to compute light index needs to be identical as in DeferredLightPixelShader.usf / ClusteredDeferredShadingPixelShader.usf
	const uint2 LocalPixelPos = PixelCoord - View.ViewRectMinAndSize.xy;
	const FCulledLightsGridHeader CulledLightGridHeader = VirtualShadowMapGetLightsGridHeader(LocalPixelPos, SceneDepth);
	const uint LightCount = min(GetPackedShadowMaskMaxLightCount(), CulledLightGridHeader.NumLights);

	LOOP
	for (uint LightIndex = 0; LightIndex < LightCount; ++LightIndex)
	{
		const uint OutputIndex = SampleIndex + MaxVisibilityNodeCount * LightIndex;

		// The Virtual Shadow Remap stores directional lights first
		const FLocalLightData LightData 	 = VirtualShadowMapGetLocalLightData(CulledLightGridHeader, LightIndex);
		const float3 TranslatedLightPosition = UnpackLightTranslatedWorldPosition(LightData);

		// Early out when the shadow mask bits are zero
		if (UnpackShadowMask(PackedShadowMask, LightIndex) == 0)
		{
			OutTransmittanceMask[OutputIndex] = PackTransmittanceMask(InitHairTransmittanceMask());
			continue;
		}

		// Early out when mismatching light channel
		if ((Sample.LightChannelMask & UnpackLightingChannelMask(LightData)) == 0)
		{
			OutTransmittanceMask[OutputIndex] = InitNullPackedHairTransmittanceMask();
			continue;
		}

		FTransmittanceSettings Settings;
		Settings.bIsDirectional				= false;
		Settings.LightDirection				= normalize(TranslatedLightPosition - TranslatedWorldPosition);
		Settings.TranslatedLightPosition	= TranslatedLightPosition;
		Settings.DeepShadowDensityScale		= 0;	// Use virtual voxel struct value
		Settings.DeepShadowDepthBiasScale	= 0;	// Use virtual voxel struct value
		Settings.Random						= Random;

	#if DEBUG_ENABLE
		const bool bDebugEnabled = PixelCoord.x == GetCursorPos().x && PixelCoord.y == GetCursorPos().y;
	#else
		const bool bDebugEnabled = false;
	#endif
		const FHairTransmittanceMask Out = ComputeTransmittanceVoxel(
			PixelCoord, 
			PixelRadius, 
			SceneDepth,
			TranslatedWorldPosition,
			MacroGroupIndex, 
			Settings, 
			bDebugEnabled);

		OutTransmittanceMask[OutputIndex] = PackTransmittanceMask(Out);
	}
}
#else // PERMUTATION_ONE_PASS
{
	const uint SampleIndex = DispatchThreadId.x + DispatchThreadId.y * TRANSMITTANCE_GROUP_SIZE * HAIR_VISIBILITY_GROUP_COUNT_WIDTH;

	const uint MaxVisibilityNodeCount = HairStrands.HairSampleCount[0];
	if (SampleIndex >= MaxVisibilityNodeCount)
	{
		return;
	}

	const uint2 PixelCoord = HairStrands.HairSampleCoords[SampleIndex];
	if (dot(RayMarchMaskTexture.Load(uint3(PixelCoord, 0)), ShadowChannelMask) == 0)
	{
		// This pixel is already fully shadowed opaque. (RayMarchMaskTexture is computed using the closest sample as of today)
		OutTransmittanceMask[SampleIndex] = InitNullPackedHairTransmittanceMask();
		return;
	}

	#if VOXEL_TRAVERSAL_DEBUG
	const bool bDebugEnabled = PixelCoord.x == GetCursorPos().x && PixelCoord.y == GetCursorPos().y;
	#else
	const bool bDebugEnabled = false;
	#endif	
	const float2 UV = (PixelCoord.xy + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	{
		const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[SampleIndex]);
		if ((LightChannelMask & Sample.LightChannelMask) == 0)
		{
			OutTransmittanceMask[SampleIndex] = InitNullPackedHairTransmittanceMask();
			return;
		}

		const float SceneDepth = ConvertFromDeviceZ(Sample.Depth);
		float3 TranslatedWorldPosition = GetTranslatedWorldPosition(UV, SceneDepth);
		if (View.StereoPassIndex == 1)
		{
			TranslatedWorldPosition += VirtualVoxel.TranslatedWorldOffsetStereoCorrection;
		}

		const uint MacroGroupIndex = clamp(Sample.MacroGroupId, 0, MAX_HAIR_MACROGROUP_COUNT - 1);

		FTransmittanceSettings Settings;
		Settings.bIsDirectional = IsLightDirectional(TranslatedLightPosition_LightDirection);
		Settings.LightDirection = GetTranslatedLightDirection(TranslatedLightPosition_LightDirection, TranslatedWorldPosition);
		Settings.TranslatedLightPosition = GetTranslatedLightPosition(TranslatedLightPosition_LightDirection, TranslatedWorldPosition);
		Settings.DeepShadowDensityScale = 0;	// Use virtual voxel struct value
		Settings.DeepShadowDepthBiasScale = 0;	// Use virtual voxel struct value
		Settings.Random = GetHairVoxelJitter(PixelCoord.xy, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);

		const float PixelRadius = ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, SceneDepth);
		const FHairTransmittanceMask Out = ComputeTransmittanceVoxel(PixelCoord, PixelRadius, SceneDepth, TranslatedWorldPosition, MacroGroupIndex, Settings, bDebugEnabled);

		#if VOXEL_TRAVERSAL_DEBUG
		if (bDebugEnabled)
		{
			PrintInfo(SampleIndex, PixelCoord, Out);
		}
		#endif	

		OutTransmittanceMask[SampleIndex] = PackTransmittanceMask(Out);
	}
}
#endif // PERMUTATION_ONE_PASS
#endif // SHADER_TRANSMITTANCE_VOXEL

////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_CLEAR

uint ElementCount;
RWBuffer<uint> OutputMask;

[numthreads(64, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint Index = DispatchThreadId.x;
	if (Index >= ElementCount)
	{
		return;
	}

	OutputMask[Index] = PackTransmittanceMask(InitHairTransmittanceMask());
}

#endif // SHADER_CLEAR
