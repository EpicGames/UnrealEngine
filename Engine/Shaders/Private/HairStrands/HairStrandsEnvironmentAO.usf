// Copyright Epic Games, Inc. All Rights Reserved.
  
#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "../DeferredShadingCommon.ush"
#include "../PositionReconstructionCommon.ush"
#include "../HairShadingCommon.ush"
#include "../HairBsdf.ush"
#include "HairStrandsEnvironmentLightingCommon.ush"
#include "../Substrate/Substrate.ush"

#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#define VOXEL_TRAVERSAL_DEBUG 0 
#include "HairStrandsVoxelPageTraversal.ush"

Texture2D<uint4> HairCategorizationTexture;

uint  Voxel_MacroGroupId;
float Voxel_TanConeAngle;
float AO_Power;
float AO_Intensity;
uint  AO_SampleCount;
float AO_DistanceThreshold;

uint Output_bHalfRes;
float2 Output_InvResolution;

float TraceAO(float3 TranslatedWorldPosition, float3 WorldNormal, float2 SvPosition, float PixelRadius)
{
	const bool bDebugEnabled = all(uint2(SvPosition) == uint2(GetCursorPos()));
	
	float3 UnoccludedN = 0;
	uint TotalHairCount = 0;
#if PERMUTATION_SAMPLESET == 0
	// Poisson disk position http://developer.download.nvidia.com/whitepapers/2008/PCSS_Integration.pdf
	float2 PoissonDisk[16] =
	{
		float2(-0.94201624, -0.39906216),
		float2(0.94558609, -0.76890725),
		float2(-0.094184101, -0.92938870),
		float2(0.34495938, 0.29387760),
		float2(-0.91588581, 0.45771432),
		float2(-0.81544232, -0.87912464),
		float2(-0.38277543, 0.27676845),
		float2(0.97484398, 0.75648379),
		float2(0.44323325, -0.97511554),
		float2(0.53742981, -0.47373420),
		float2(-0.26496911, -0.41893023),
		float2(0.79197514, 0.19090188),
		float2(-0.24188840, 0.99706507),
		float2(-0.81409955, 0.91437590),
		float2(0.19984126, 0.78641367),
		float2(0.14383161, -0.14100790)
	};
	const uint SampleCount = clamp(AO_SampleCount, 1u, 16u);
#else
	const uint SampleCount = max(AO_SampleCount, 1u);
#endif

	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution = VirtualVoxel.PageCountResolution;
	CommonDesc.PageTextureResolution = VirtualVoxel.PageTextureResolution;
	CommonDesc.PageResolution = VirtualVoxel.PageResolution;
	CommonDesc.PageResolutionLog2 = VirtualVoxel.PageResolutionLog2;

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[Voxel_MacroGroupId], VirtualVoxel.PageResolution);

	FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
	TraversalSettings.DensityScale = VirtualVoxel.DensityScale_AO;
	TraversalSettings.CountThreshold = 1;
	TraversalSettings.DistanceThreshold = AO_DistanceThreshold;
	TraversalSettings.bDebugEnabled = bDebugEnabled;
	TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Environment;
	TraversalSettings.bUseOpaqueVisibility = false;
	const float VoxelAOMaxDistance = 1000.0;
	float3 BentNormal = 0;
	float Visibility = 0;
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float4 Random4 = ComputeRandom4(SvPosition, SampleIt, SampleCount, View.StateFrameIndexMod8);

		const float3 LocalL = CosineSampleHemisphere(frac(Random4.xy)).xyz;
		const float3 SampleL = TangentToWorld(LocalL.xyz, WorldNormal);
		const float3 SampleBias = ComputePositionBias(SampleL, SampleL * Random4.z, NodeDesc.VoxelWorldSize, VirtualVoxel.DepthBiasScale_Environment);

		// Depth bias
		const float3 SampleTranslatedWorldPosition = TranslatedWorldPosition + SampleBias;
		const float3 SampleTranslatedLightPosition = SampleTranslatedWorldPosition + SampleL * VoxelAOMaxDistance;

		// Compute the number of hair count between light & shading point

		const FHairTraversalResult Result = ComputeHairCountVirtualVoxel(
			SampleTranslatedWorldPosition,
			SampleTranslatedLightPosition,
			CommonDesc,
			NodeDesc,
			VirtualVoxel.PageIndexBuffer,
			VirtualVoxel.PageTexture,
			TraversalSettings);

		const float StepVisibility = saturate(1 - Result.HairCount) * Result.Visibility;
		BentNormal += SampleL * StepVisibility;
		Visibility += StepVisibility;
	}
	Visibility /= SampleCount;
	BentNormal /= SampleCount;

	// User adjust AO
	const float AO = 1 - (1 - pow(saturate(Visibility), AO_Power)) * AO_Intensity;
	return saturate(AO);
}

void MainPS(
	in float4 SvPosition : SV_Position,
	out float4 OutAO : SV_Target0)
{
	const bool bHalfRes = true; // Output_bHalfRes > 0;
	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	if (bHalfRes > 0)
	{
		BufferUV = SvPosition.xy * Output_InvResolution;
	}

	const uint2 PixelCoord = floor(SvPosition.xy);
#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));
	const bool bIsValid = SubstratePixelHeader.ClosureCount > 0;
	const float3 WorldNormal = TopLayerData.WorldNormal;

#else
	// Trace AO for all pixel which are partially covered or not covered at all.
	// Fully covered pixel are marked as unlit
	const FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);
	const bool bIsValid = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
	const float3 WorldNormal = GBuffer.WorldNormal;
#endif

	OutAO = 1;
	if (bIsValid)
	{
		const float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
		const float SceneDepth = ConvertFromDeviceZ(DeviceZ);
		const float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDepth(BufferUV, SceneDepth);
		const float PixelRadius = ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, SceneDepth);
		OutAO = TraceAO(TranslatedWorldPosition, WorldNormal, SvPosition.xy, PixelRadius);
	}
	else
	{
		discard;
	}
}
