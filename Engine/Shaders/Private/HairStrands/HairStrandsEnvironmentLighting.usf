// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReflectionEnvironmentComputeShaders - functionality to apply local cubemaps.
=============================================================================*/

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Note: 
// * This version of environement lighting is very much in WIP state, with a lot of option to tune tweak to explore
//   what kind of approximation works the best for certains type of grooms and lighting condition
// * There is a few fudges factor as well (roughness mapping, TT density, reflected vector orientations, ...)

// Sky lighting
// ============
// Note: Strands & Cards are always dynamic primitive
// 
//          | Static    | Stationnary  | Dynamic
// ---------|-----------|--------------|--------
// Cards    | BasePass  | BasePass     | EnvPass
// Strands  | EnvPass   | EnvPass      | EnvPass

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Includes

// Override ALLOW_STATIC_LIGHTING based on shader permutation
// Need to be define prior to other includes
#undef ALLOW_STATIC_LIGHTING
#if PERMUTATION_STATIC_LIGHTING
	#define ALLOW_STATIC_LIGHTING 1
	#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 1
	#define PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING 1
	#define APPLY_SKY_SHADOWING 1
#else
	#define ALLOW_STATIC_LIGHTING 0
	#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 0
	#define PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING 0
#endif
#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES 1
#define REFLECTION_COMPOSITE_HAS_BOX_CAPTURES 1
#define REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES 1
#define ENABLE_SKY_LIGHT 1
#define USE_HAIR_COMPLEX_TRANSMITTANCE 1
#define ADAPTIVE_PROBES 0

#include "../Common.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"
#include "HairStrandsCommon.ush"
#include "../DeferredShadingCommon.ush"  
#include "../BRDF.ush"
#include "../ReflectionEnvironmentShared.ush"
#include "../SkyLightingShared.ush"
#include "../SkyLightingDiffuseShared.ush"
#include "../DistanceFieldAOShared.ush"
#include "../ShadingModels.ush"
#include "../LightGridCommon.ush"
#include "../SceneTextureParameters.ush"
#include "../ReflectionEnvironmentComposite.ush"
#include "../SHCommon.ush"
#include "../ReflectionEnvironmentShared.ush"
#include "../SkyLightingShared.ush"
#include "../VolumetricLightmapShared.ush"
#include "../HairBsdf.ush"
#include "../ShaderPrint.ush"
#if PERMUTATION_LUMEN == 1
#include "../Lumen/LumenRadianceCacheCommon.ush"
#include "../Lumen/LumenMaterial.ush"
#endif
#if PERMUTATION_LUMEN == 2
#include "../Lumen/LumenTracingCommon.ush"
#include "../Lumen/LumenIrradianceFieldInterpolation.ush"
#include "../Lumen/LumenMaterial.ush"
#endif

#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVisibilityUtils.ush"
#include "HairStrandsVoxelPageCommon.ush"
#include "HairStrandsDeepShadowCommon.ush"
#include "HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrandsDeepTransmittanceDualScattering.ush"
#include "HairStrandsEnvironmentLightingCommon.ush"

#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
//#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR
#define VOXEL_TRAVERSAL_DEBUG 0
#include "HairStrandsVoxelPageTraversal.ush"

#define USE_BLUE_NOISE 1
#include "../BlueNoise.ush"
DEFINE_HAIR_BLUE_NOISE_JITTER_FUNCTION

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Defines

#define INTEGRATION_SCENECOLOR 0
#define INTEGRATION_ADHOC 1
#define INTEGRATION_UNIFORM 2
#define INTEGRATION_SH 3

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Parameters

uint				bDynamicSkyLight;
uint				bHasStaticLighting;
Texture3D<float4>	HairEnergyLUTTexture;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Static lighting evaluation (diffuse)
#if PERMUTATION_STATIC_LIGHTING

/** Calculates indirect lighting contribution on this object from precomputed data. */
float3 GetPrecomputedIndirectLighting(float3 AbsoluteWorldPosition, float3 DiffuseDir)
{
	float3 OutDiffuseLighting = 0;

	#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
	const float3 VolumetricLightmapBrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(AbsoluteWorldPosition);
	FThreeBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH3(VolumetricLightmapBrickTextureUVs);
	
	// Diffuse convolution
	FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(DiffuseDir, 1);
	OutDiffuseLighting = max(float3(0,0,0), DotSH3(IrradianceSH, DiffuseTransferSH)) / PI;
	#endif

	// Apply indirect lighting scale while we have only accumulated lightmaps
	OutDiffuseLighting *= View.PrecomputedIndirectLightingColorScale;

	// Note: pre-exposition is applied by the caller
	return OutDiffuseLighting;
}
#endif // PERMUTATION_STATIC_LIGHTING

float3 GetStaticLighting(in FGBufferData GBuffer, float3 CameraDirection, float3 AbsoluteWorldPosition)
{
	float3 OutStaticLighting = 0;

	#if PERMUTATION_STATIC_LIGHTING
	// Evaluation of the static lighting
	if (bHasStaticLighting)
	{
		float3 N = GBuffer.WorldNormal;
		float3 V = CameraDirection;
		float3 L = 0;
		const float3 DiffuseColorForIndirect = EvaluateEnvHair(GBuffer, V, N, L /*out*/);
		const float3 DiffuseIndirectLighting = GetPrecomputedIndirectLighting(AbsoluteWorldPosition, L);

		const float MaterialAO = 1;
		const float DiffOcclusion = MaterialAO;
		OutStaticLighting = (DiffuseIndirectLighting * DiffuseColorForIndirect) * AOMultiBounce(GBuffer.BaseColor, DiffOcclusion);
	}
	#endif
	return OutStaticLighting * View.PreExposure;
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Reflection probe evaluation
float3 GatherRadiance(float3 RayDirection, float Roughness, uint2 PixelPosition, float PixelSceneDepth, float3 TranslatedWorldPosition)
{
	const float ConeHalfAngle = saturate(Roughness * .8f);
#if PERMUTATION_LUMEN == 1
	// Lumen probe
	FRadianceCacheCoverage Coverage = GetRadianceCacheCoverage(TranslatedWorldPosition, RayDirection, .5f);
	if (Coverage.bValid)
	{
		return SampleRadianceCacheInterpolated(Coverage, TranslatedWorldPosition, RayDirection, ConeHalfAngle, /*RandomScalarForStochasticInterpolation*/ 0.5f).Radiance;
	}	
	return 0;
#elif PERMUTATION_LUMEN == 2
	{
		const float3 Material_WorldNormal = RayDirection;
		const float3 WorldNormalForReflection = Material_WorldNormal;

		float3 DiffuseLighting = 0;
		float3 SpecularLighting = 0;
		GetIrradianceFieldInterpolatedLighting(
			PixelPosition,
			TranslatedWorldPosition,
			Material_WorldNormal,
			Roughness,
			WorldNormalForReflection,
			false /*bHasBackfaceDiffuse*/,
			false /*bPixelShouldProcessFeedback*/,
			DiffuseLighting, 
			SpecularLighting);
		return SpecularLighting;
	}
#else
	const float IndirectIrradiance = 1;
	const int SingleCaptureIndex = 0;

	const uint EyeIndex = 0;
	const uint GridIndex = ComputeLightGridCellIndex(PixelPosition, PixelSceneDepth, EyeIndex);

	FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);

	const bool bCompositeSkylight = true;
	return CompositeReflectionCapturesAndSkylightTWS(
		1.0f,
		TranslatedWorldPosition,
		RayDirection,
		Roughness,
		IndirectIrradiance,
		1.0f,
		0.0f,
		CulledReflectionCapturesGridHeader.NumReflectionCaptures,
		CulledReflectionCapturesGridHeader.DataStartIndex,
		SingleCaptureIndex,
		bCompositeSkylight);
#endif // PERMUTATION_LUMEN
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Debug info

#if PERMUTATION_DEBUG
#include "HairStrandsDebugCommonStruct.ush"

RWStructuredBuffer<FDebugShadingInfo>	Debug_ShadingPointBuffer;
RWBuffer<uint>							Debug_ShadingPointCounter;
RWStructuredBuffer<FDebugSample>		Debug_SampleBuffer;
RWBuffer<uint>							Debug_SampleCounter;
uint									Debug_MaxShadingPointCount;
uint									Debug_MaxSampleCount;

uint AddShadingPoint(const FGBufferData GB, const float3 V, uint SampleCount)
{
	uint SampleOffset = 0;
	InterlockedAdd(Debug_SampleCounter[0], SampleCount, SampleOffset);

	uint PointOffset = 0;
	InterlockedAdd(Debug_ShadingPointCounter[0], 1, PointOffset);
	if (PointOffset < Debug_MaxShadingPointCount)
	{
		FDebugShadingInfo Out;
		Out.BaseColor = GB.BaseColor;
		Out.Roughness = GB.Roughness;
		Out.V = V;
		Out.T = GB.WorldNormal;
		Out.SampleCount = SampleCount;
		Out.SampleOffset = SampleOffset;
		Debug_ShadingPointBuffer[PointOffset] = Out;
	}

	return SampleOffset;
}

void AddSample(uint SampleOffset, uint SampleIt, const float3 D, const float3 W)
{
	FDebugSample Out = (FDebugSample)0;
	Out.Direction = D;
	Out.Weight = W;
	Out.Pdf = 1;
	Debug_SampleBuffer[SampleOffset + SampleIt] = Out;
}
#endif //PERMUTATION_DEBUG

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

float HairDistanceThreshold;
uint bHairUseViewHairCount;
Texture2D<float> HairCountTexture;

uint  MultipleScatterSampleCount;
float TransmissionDensityScaleFactor;

FHairTraversalResult InternalComputeHairCountVoxel(
	const FVirtualVoxelNodeDesc NodeDesc,
	float3 TranslatedWorldPosition, 
	float3 TranslatedLightPosition, 
	float ExtraDensityScale, 
	float CountThreshold, 
	float DistanceThreshold,
	float PixelRadius, 
	bool bUseConeTracing,
	bool bDebugEnabled)
{
	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution = VirtualVoxel.PageCountResolution;
	CommonDesc.PageTextureResolution = VirtualVoxel.PageTextureResolution;
	CommonDesc.PageResolution = VirtualVoxel.PageResolution;
	CommonDesc.PageResolutionLog2 = VirtualVoxel.PageResolutionLog2;

	FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
	TraversalSettings.DensityScale = VirtualVoxel.DensityScale_Environment * ExtraDensityScale;
	TraversalSettings.CountThreshold = GetOpaqueVoxelValue();
	TraversalSettings.DistanceThreshold = DistanceThreshold;
	TraversalSettings.bDebugEnabled = bDebugEnabled;
	TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Environment;
	TraversalSettings.PixelRadius = PixelRadius;

	return ComputeHairCountVirtualVoxel(
		TranslatedWorldPosition,
		TranslatedLightPosition,
		CommonDesc,
		NodeDesc,
		VirtualVoxel.PageIndexBuffer,
		VirtualVoxel.PageTexture,
		TraversalSettings);
}

#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_UNIFORM || PERMUTATION_INTEGRATION_TYPE == INTEGRATION_ADHOC || PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SH
#include "HairStrandsScreenTracing.ush"
uint bScreenTrace;
#endif

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_ADHOC

#define ADHOC_SAMPLING_R 1
#define ADHOC_SAMPLING_TT 0
#define ADHOC_SAMPLING_TRT 1
#define ADHOC_SAMPLING_MS 1

float MaxComponent3(float3 In)
{
	return max(In.x, max(In.y, In.z));
}

float3 ReflectionEnvironment(FGBufferData GBuffer, float2 ScreenPosition, float2 SvPosition, uint ControlPointId, uint MacroGroupId)
{
	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	const uint2  PixelPosition = SvPosition - ResolvedView.ViewRectMin.xy;
	const float  PixelSceneDepth = GBuffer.Depth;
	const float  PixelRadius = ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, PixelSceneDepth);

	const uint ControlPointIdMod8 = (ControlPointId % 8);
	const float3 VoxelOffset =  GetHairVoxelJitter(SvPosition, View.StateFrameIndexMod8 + ControlPointIdMod8, VirtualVoxel.JitterMode);

	const float3 V = -CameraToPixel;
	const float3 T = GBuffer.WorldNormal;
	const bool bDebugEnabled = all(PixelPosition == uint2(GetCursorPos()));

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[MacroGroupId], VirtualVoxel.PageResolution);

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Trace/sample transmittance in various direction to have estimate of where is occlusion & computing the integrated dual scattering value
	float3 UnoccludedN = -V;
	float3 AvgDualScattering = 0;
	const float MaxSearchDistance = HairDistanceThreshold; // in cm
	#if ADHOC_SAMPLING_MS || ADHOC_SAMPLING_R || ADHOC_SAMPLING_TT || ADHOC_SAMPLING_TRT
	if ((View.HairComponents & HAIR_COMPONENT_R) || (View.HairComponents & HAIR_COMPONENT_TRT) || (View.HairComponents & HAIR_COMPONENT_TT))
	{
		#if PERMUTATION_SAMPLESET == 0
		// Poisson disk position http://developer.download.nvidia.com/whitepapers/2008/PCSS_Integration.pdf
		float2 PoissonDisk[16] =
		{
			float2(-0.94201624, -0.39906216),
			float2(0.94558609, -0.76890725),
			float2(-0.094184101, -0.92938870),
			float2(0.34495938, 0.29387760),
			float2(-0.91588581, 0.45771432),
			float2(-0.81544232, -0.87912464),
			float2(-0.38277543, 0.27676845),
			float2(0.97484398, 0.75648379),
			float2(0.44323325, -0.97511554),
			float2(0.53742981, -0.47373420),
			float2(-0.26496911, -0.41893023),
			float2(0.79197514, 0.19090188),
			float2(-0.24188840, 0.99706507),
			float2(-0.81409955, 0.91437590),
			float2(0.19984126, 0.78641367),
			float2(0.14383161, -0.14100790)
		};
		const uint SampleCount = clamp(MultipleScatterSampleCount, uint(1), uint(16));
		#else
		const uint SampleCount = max(MultipleScatterSampleCount, uint(1));
		#endif

		float3 AvgR = 0;
		for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			const float4 Random4 = ComputeRandom4(PixelPosition, SampleIt, SampleCount, View.StateFrameIndexMod8);

			const float3 SampleL = UniformSampleSphere(frac(Random4.xy)).xyz;
			const float3 SampleBias = ComputePositionBias(SampleL, SampleL * Random4.z, NodeDesc.VoxelWorldSize, VirtualVoxel.DepthBiasScale_Environment);
			const float3 SampleTranslateWorldPosition = TranslatedWorldPosition + SampleBias;
			const float3 SampleTranslateLightPosition = SampleTranslateWorldPosition + SampleL* MaxSearchDistance;

			// Compute the number of hair count between light & shading point
			const float SinLightAngle = dot(SampleL, T);
			const FHairTraversalResult Result = InternalComputeHairCountVoxel(NodeDesc, SampleTranslateWorldPosition, SampleTranslateLightPosition, 1, 0, MaxSearchDistance, PixelRadius, true, bDebugEnabled);

			const uint ScatteringComponent = (View.HairComponents & HAIR_COMPONENT_R) | (View.HairComponents & HAIR_COMPONENT_TRT) | HAIR_COMPONENT_LS | HAIR_COMPONENT_GS;// | HAIR_MULTISCATTER;
			const FHairTransmittanceData TransmittanceData = GetHairTransmittance(V, SampleL, GBuffer, InitHairTransmittanceMask(Result.HairCount, Result.Visibility), View.HairScatteringLUTTexture, HairScatteringLUTSampler, ScatteringComponent);

			const float Area = 0.2;
			const float BackLit = 1;
			//AvgDualScattering += HairShading(GBuffer, SampleL, V, T, 1, TransmittanceData, BackLit, Area, uint2(0, 0));
			AvgDualScattering += TransmittanceData.LocalScattering * TransmittanceData.GlobalScattering * Result.Visibility;

			const float TransmittedVisibility = MaxComponent3(TransmittanceData.GlobalScattering * Result.Visibility);
			AvgR += SampleL * TransmittedVisibility;
		}
		const float SphereInt = 4 * PI;
		AvgDualScattering *= SphereInt;
		AvgDualScattering /= SampleCount;


		const float NormR = sqrt(dot(AvgR, AvgR));
		UnoccludedN = AvgR / NormR;
		
		#if VOXEL_TRAVERSAL_DEBUG
		if (bDebugEnabled)
		{
			AddLineTWS(TranslatedWorldPosition, TranslatedWorldPosition + UnoccludedN * 50, float4(1, 0, 1, 1), float4(1, 0, 1, 1));
		}
		#endif

		// For debug purpose:
		//const float UnoccludedVarianceR = NormR * (3 - NormR2) / (1 - NormR2);
		//const float3 ColorDir = (UnoccludedN+ float3(1, 1, 1)) * 0.5f;
		//return ColorDir.rgb;
		//return View.PreExposure * GetSkySHDiffuse(UnoccludedN) * View.SkyLightColor.rgb;
		//return View.PreExposure * GatherRadiance(UnoccludedN, GBuffer.Roughness);
		//return AvgDualScattering;
	}
	#endif
	
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// MS ==============================================================================
	float3 ColorMS = 0;
	#if ADHOC_SAMPLING_MS
	if ((View.HairComponents & HAIR_COMPONENT_R) || (View.HairComponents & HAIR_COMPONENT_TRT) || (View.HairComponents & HAIR_COMPONENT_TT))
	{
		float3 N = UnoccludedN;
		float3 R = 2 * dot(V, N) * N - V;
		R = UnoccludedN;

		#if 1
		{
			N = GBuffer.WorldNormal;
			N = normalize(V - N * dot(V, N));
			float3 R_b = 2 * dot(V, N) * N - V;
			const float alpha = 0.4f;
			R = normalize(lerp(R_b, R, alpha));
		}
		#endif	

		// Select the TT term for representing the MS roughness lookup as TT lobe carries the most energy + fudge scale as the MS lobe should be wider than a TT lobe
		const float FudgeScale = 2;
		const float GGXRoughness = HairToGGXRoughness_TT(GBuffer.Roughness) * FudgeScale;
		ColorMS.rgb += View.PreExposure * GatherRadiance(R, GGXRoughness, PixelPosition, PixelSceneDepth, TranslatedWorldPosition);
		//ColorMS.rgb += View.PreExposure * GetSkySHDiffuse(R) * View.SkyLightColor.rgb;
		ColorMS.rgb *= AvgDualScattering;

		// Directional albedo
		#if 0
		{
			const float SinViewAngle = dot(T, R);
			FHairAverageEnergy Energy = SampleHairEnergyLUT(HairEnergyLUTTexture, HairScatteringLUTSampler, GBuffer.BaseColor, GBuffer.Roughness, SinViewAngle);
			ColorMS.rgb *= Energy.A_R + Energy.A_TT + Energy.A_TRT;
		}
		#endif	
	}
	#endif

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// R ==============================================================================
	float3 ColorR = 0;
	#if ADHOC_SAMPLING_R || ADHOC_SAMPLING_TRT
	if ((View.HairComponents & HAIR_COMPONENT_R) || (View.HairComponents & HAIR_COMPONENT_TRT))
	{
		float3 R = 0;
		{
			// This is not correct for hair rendering
			float3 N = UnoccludedN;
			R = 2 * dot( V, N ) * N - V;

			// Use a fake normal for computing the reflected vector instead
			#if 1
			{
				N = GBuffer.WorldNormal;
				N = normalize(V - N * dot(V, N));
				float3 R_b = 2 * dot(V, N) * N - V;
				const float alpha = 0.4f;
				R = normalize(lerp(R_b, R, alpha));
			}
			#endif	

		}

		// Select the TRT for remapping the hair roughness onto the GGX based lobe, as it is the 
		// widest lobe between the R & TRT components
		const float GGXRoughness = HairToGGXRoughness_TRT(GBuffer.Roughness);
		ColorR.rgb += View.PreExposure * GatherRadiance(R, GGXRoughness, PixelPosition, PixelSceneDepth, TranslatedWorldPosition);
		//ColorR.rgb += View.PreExposure * GetSkySHDiffuse(R) * View.SkyLightColor.rgb * PI;

		const float3 L = R;
		const float SinLightAngle = dot(L, T);
		const float SinViewAngle = dot(V, T);

		// Compute the transmittance
		float3 Transmittance = 1;
		float Visibility = 1;
		{
			const float3 SampleBias = ComputePositionBias(L, VoxelOffset, NodeDesc.VoxelWorldSize, VirtualVoxel.DepthBiasScale_Environment);
			const float3 SampleTranslatedWorldPosition = TranslatedWorldPosition + SampleBias;
			const float3 SampleTranslatedLightPosition = SampleTranslatedWorldPosition + L * MaxSearchDistance;

			// Compute the number of hair count between light & shading point
			const FHairTraversalResult Result = InternalComputeHairCountVoxel(NodeDesc, SampleTranslatedWorldPosition, SampleTranslatedLightPosition, 1, 0, MaxSearchDistance, PixelRadius, true, bDebugEnabled);
			Transmittance = GetHairTransmittanceOnly(InitHairTransmittanceMask(Result.HairCount, Result.Visibility), GBuffer, SinLightAngle, View.HairScatteringLUTTexture, HairScatteringLUTSampler);
			Visibility = Result.Visibility;
		}


		// Directional albedo
		#if 0
		{
			FHairAverageEnergy Energy = SampleHairEnergyLUT(HairEnergyLUTTexture, HairScatteringLUTSampler, GBuffer.BaseColor, GBuffer.Roughness, SinViewAngle);
			ColorR.rgb *= (Energy.A_TRT + Energy.A_R) * Transmittance.xyz * Visibility;
		}
		#endif

		// BSDF
		#if 1
		{
			FHairTransmittanceData TransmittanceData = InitHairStrandsTransmittanceData();
			TransmittanceData.OpaqueVisibility = Visibility;
			TransmittanceData.GlobalScattering = Transmittance;
			TransmittanceData.ScatteringComponent = (View.HairComponents & HAIR_COMPONENT_R) | (View.HairComponents & HAIR_COMPONENT_TRT) | HAIR_COMPONENT_MULTISCATTER;

			const float Area = 0.2;
			const float BackLit = 0;
			ColorR.rgb *= HairShading(GBuffer, L, V, T, 1, TransmittanceData, BackLit, Area, uint2(0, 0)) * Visibility;
		}
		#endif
	}
	#endif // R TRT

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// TT ==============================================================================
	float3 ColorTT = 0;
	#if ADHOC_SAMPLING_TT
	if (View.HairComponents & HAIR_COMPONENT_TT)
	{
		const float3 L = -V;
		const float SinLightAngle = dot(L, T);
		const float SinViewAngle = dot(V, T);

		const float GGXRoughness = HairToGGXRoughness_TT(GBuffer.Roughness);
		ColorTT.rgb += View.PreExposure * GatherRadiance(L, GGXRoughness, PixelPosition, PixelSceneDepth, TranslatedWorldPosition);
		//ColorTT.rgb += View.PreExposure * GetSkySHDiffuse(L) * View.SkyLightColor.rgb;

		// Compute the transmittance
		float3 Transmittance = 1;
		float Visibility = 1;
		{
			const float3 SampleBias = ComputePositionBias(L, VoxelOffset, NodeDesc.VoxelWorldSize, VirtualVoxel.DepthBiasScale_Environment);
			const float3 SampleTranslatedWorldPosition = TranslatedWorldPosition + SampleBias;
			const float3 SampleTranslatedLightPosition = SampleTranslatedWorldPosition + L * MaxSearchDistance;

			// Compute the number of hair count between light & shading point
			// For TT, we don't want to abort the hair counting, as this is important to get the proper transmission
			const float DistanceThresholdTT = 100000;
			FHairTraversalResult Result;
			if (bHairUseViewHairCount)
			{
				Result.HairCount = HairCountTexture.Load(uint3(SvPosition.xy, 0)) * max(TransmissionDensityScaleFactor, 0);
				Result.Visibility = 1;
			}
			else
			{
				Result = InternalComputeHairCountVoxel(NodeDesc, SampleTranslatedWorldPosition, SampleTranslatedLightPosition, TransmissionDensityScaleFactor, 0, DistanceThresholdTT, PixelRadius, false, bDebugEnabled);
			}
			Transmittance = GetHairTransmittanceOnly(Result.HairCount, Result.Visibility, GBuffer, SinLightAngle, View.HairScatteringLUTTexture, HairScatteringLUTSampler);
			Visibility = Result.Visibility;
		}

		// BSDF with dual-scattering
		#if 0
		{
			Error the Transmittance.w does not contain the correct hair count when we hit opaque geometry: TODO
			const uint HairComponent = HAIR_COMPONENT_TT | HAIR_COMPONENT_LS | HAIR_COMPONENT_GS;
			const FHairTransmittanceData TransmittanceData = GetHairTransmittance(V, L, GBuffer, Transmittance, View.HairScatteringLUTTexture, HairScatteringLUTSampler, HairComponent);
			const float Area = 0.2;
			ColorTT.rgb *= HairShading(GBuffer, L, V, T, 1, TransmittanceData, 1, Area, uint2(0, 0)) * Visibility;
		}
		#endif

		// Average weighting
		#if 1
		{
			const float Backlit = GBuffer.CustomData.z;
			FHairAverageEnergy Energy = SampleHairEnergyLUT(HairEnergyLUTTexture, HairScatteringLUTSampler, GBuffer.BaseColor, GBuffer.Roughness, SinViewAngle);
			ColorTT.rgb *= Energy.A_TT * Transmittance * Visibility * Backlit;
		}
		#endif	

		// BSDF
		#if 0
		{
			FHairTransmittanceData TransmittanceData = InitHairStrandsTransmittanceData();
			TransmittanceData.MultipleScattering = 1;
			TransmittanceData.GlobalScattering = Transmittance.xyz;
			TransmittanceData.LocalScattering = 0;
			TransmittanceData.ScatteringComponent = HAIR_COMPONENT_TT | HAIR_COMPONENT_MULTISCATTER;

			const float Area = 0.2;
			ColorTT.rgb *= HairShading(GBuffer, L, V, T, 1, TransmittanceData, 0, Area, uint2(0, 0)) * Visibility;
		}
		#endif	
	}
	#endif // TT

	const float3 Color = ColorR + ColorTT + ColorMS;

	// Transform NaNs to black, transform negative colors to black.
	return -min(-Color.rgb, 0.0);
}
#endif // PERMUTATION_INTEGRATION_TYPE == INTEGRATION_ADHOC

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Reference light integration using uniform spherical sampling

#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_UNIFORM
float3 ReflectionEnvironmentUniformSampling(
	FGBufferData GBuffer, 
	float2 ScreenPosition, 
	float2 SvPosition, 
	uint HairControlPointId, 
	uint MacroGroupId,
	bool bDebugEnabled)
{
	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	const uint2  PixelPosition = SvPosition - ResolvedView.ViewRectMin.xy;
	const float  PixelSceneDepth = GBuffer.Depth;
	const float  PixelRadius = ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, PixelSceneDepth);

	const float3 V = -CameraToPixel;
	const float3 T = GBuffer.WorldNormal;

	// Trace/sample transmittance in various direction to have estimate of where is occlusion & computing the integrated dual scattering value
	const float MaxSearchDistance = HairDistanceThreshold; // in cm
	const uint SampleCount = max(MultipleScatterSampleCount, uint(1));

	#if PERMUTATION_DEBUG
	uint SampleOffset = 0;
	if (bDebugEnabled)
	{
		SampleOffset = AddShadingPoint(GBuffer, V, SampleCount);
	}
	#endif

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[MacroGroupId], VirtualVoxel.PageResolution);

	float3 AccLighting = 0;
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		#if USE_BLUE_NOISE
		const float4 Random4 = GetHairBlueNoiseJitter(PixelPosition, SampleIt, SampleCount, View.StateFrameIndexMod8);
		#else
		const float4 Random4 = ComputeRandom4(PixelPosition, SampleIt, SampleCount, View.StateFrameIndexMod8);
		#endif

		// Depth bias
		const float3 SampleL    = UniformSampleSphere(frac(Random4.xy)).xyz;
		const float3 SampleBias = ComputePositionBias(SampleL, Random4.z * SampleL, NodeDesc.VoxelWorldSize, VirtualVoxel.DepthBiasScale_Environment);

		const float3 SampleTranslatedWorldPosition = TranslatedWorldPosition + SampleBias;
		const float3 SampleTranslatedLightPosition = SampleTranslatedWorldPosition + SampleL * MaxSearchDistance;

		#if PERMUTATION_DEBUG
		if (bDebugEnabled)
		{
			AddSample(SampleOffset, SampleIt, SampleL, 1);
		}
		#endif

		const float SinLightAngle = dot(SampleL, T);
		FHairTraversalResult Result = InternalComputeHairCountVoxel(
			NodeDesc,
			SampleTranslatedWorldPosition, 
			SampleTranslatedLightPosition, 
			1,
			0, 
			MaxSearchDistance, 
			PixelRadius,
			true, 
			false/*bDebugEnabled*/);

		FHairTransmittanceData TransmittanceData = GetHairTransmittance(
			V,
			SampleL,
			GBuffer,
			InitHairTransmittanceMask(Result.HairCount, Result.Visibility),
			View.HairScatteringLUTTexture, 
			HairScatteringLUTSampler,
			View.HairComponents);	

		if (Result.Visibility > 0 && bScreenTrace)
		{
			Result.Visibility *= GetRayVisibility(TranslatedWorldPosition, PixelSceneDepth, SampleL, PixelPosition, SampleIt, SampleCount, VirtualVoxel.JitterMode > 0);
		}

		const float Area = 0;	
		const float Backlit = 1;
		const float RoughnessBias = 0.1f;
		AccLighting += 
			HairShading(GBuffer, SampleL, V, T, 1, TransmittanceData, Backlit, Area, uint2(0, 0)) * 
			Result.Visibility * 
			GatherRadiance(SampleL, RoughnessBias, PixelPosition, PixelSceneDepth, TranslatedWorldPosition);

		#if PERMUTATION_DEBUG
		if (bDebugEnabled)
		{
			AddLineTWS(SampleTranslatedWorldPosition, SampleTranslatedLightPosition, float4(1, 0, 1, 1), float4(1, 0, 1, 1));
		}
		#endif
	}
	const float SphereInt = 4 * PI;
	float3 FinalColor = SphereInt * AccLighting / SampleCount;
	
	FinalColor *= View.PreExposure;

	// Transform NaNs to black, transform negative colors to black.
	return -min(-FinalColor, 0.0);
}
#endif // #if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_UNIFORM

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Spherical based harmonic integration

#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SH

float3 ReflectionEnvironmentSH(
	FGBufferData GBuffer,
	float2 ScreenPosition,
	float2 SvPosition,
	uint HairControlPointId,
	uint MacroGroupId,
	bool bDebugEnabled)
{
	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 AbsoluteWorldPosition = TranslatedWorldPosition + DFHackToFloat(PrimaryView.WorldCameraOrigin);
	const float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	const float3 V = -CameraToPixel;

#if PERMUTATION_LUMEN
	const float3 T = GBuffer.WorldNormal;
	const float3 N = T;
	const float3 L = normalize(V - N * dot(V, N));

	FHairTransmittanceData TransmittanceData = GetHairTransmittance(
		V,
		L,
		GBuffer,
		InitHairTransmittanceMask(),
		View.HairScatteringLUTTexture,
		HairScatteringLUTSampler,
		View.HairComponents);

	// Note: We don't reuse EvaluateEnvHair() here as we need to estimate occlusion with GetHairTransmittance(), and the 2 * PI gives too bright result 
	// for some reason. It is set to 1 * PI for now until we get to the bottom of the issue.
	const float3 IndirectDiffuseColor = min(1.f, PI * HairShading(GBuffer, L, V, N, 1/*Shadow*/, TransmittanceData, 0.f /*Backlit*/, 0.2/*Area*/, uint2(0, 0)/*Random*/));

	// This logic mimic the ReflecitonEnvironemtPixelShader.usf. Ideallyw we should share the same
	// code/path, but the various permutation make the code a bit obscure
	// Need to take into account DFAO?
	float3 Lighting = 0;
	{
		// Probe lighting
		const uint2  PixelPosition = SvPosition - ResolvedView.ViewRectMin.xy;
		Lighting = View.PreExposure * GatherRadiance(L, GBuffer.Roughness, PixelPosition, GBuffer.Depth, TranslatedWorldPosition) * IndirectDiffuseColor;
	}

	if (bScreenTrace)
	{
		Lighting *= GetRayVisibility(TranslatedWorldPosition, GBuffer.Depth, L, SvPosition, 0, 1, VirtualVoxel.JitterMode > 0);
	}

#else // PERMUTATION_LUMEN
	const float AmbientOcclusion = 1.0f;
	const float3 BentNormal = GBuffer.WorldNormal;
	const float3 DiffuseColor = GBuffer.BaseColor; // not used
	const float2 BufferUV = SvPosition * View.BufferSizeAndInvSize.zw; // not used

	float3 Lighting = 0;

	// Sky lighting (already pre-exposed)
	if (bDynamicSkyLight)
	{
		Lighting += SkyLightDiffuse(GBuffer, AmbientOcclusion, BufferUV, ScreenPosition, BentNormal, DiffuseColor);
	}

	// Static lighting (already pre-exposed)
	if (bHasStaticLighting)
	{
		Lighting += GetStaticLighting(GBuffer, V, AbsoluteWorldPosition);
	}
#endif // PERMUTATION_LUMEN

	// Transform NaNs to black, transform negative colors to black.
	return -min(-Lighting, 0.0);
}
#endif // #if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SH

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Simple lookup of the scene color for single/low count of hair, close to the surface

#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SCENECOLOR
Texture2D<float4> SceneColorTexture;

float3 ComputeThinTransmission(FGBufferData GBuffer, float2 ScreenPosition, float2 SvPosition, uint HairControlPointId)
{
	const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	const uint2  PixelPosition = SvPosition - ResolvedView.ViewRectMin.xy;

	const float3 V = -CameraToPixel;
	const float3 T = GBuffer.WorldNormal;

	const float PixelCoverage = min(HairStrands.HairCoverageTexture.Load(uint3(PixelPosition, 0)), 1);
	
	// The scene color is attenuated propertionnaly to the hair coverage. In order to retrieve the actual lighting buffer 
	// value, we divide by the hair coverage
	const float4 OccludedLuminance = SceneColorTexture.Load(uint3(PixelPosition, 0));
	const float4 Luminance = OccludedLuminance / PixelCoverage;

	// The pre-exposure is not applied onto the output as the SceneColorTexture value are already preexposed.
	const float SinViewAngle = dot(T, V);
	const FHairAverageScattering Energy = SampleHairLUT(View.HairScatteringLUTTexture, HairScatteringLUTSampler, GBuffer.BaseColor, GBuffer.Roughness, SinViewAngle);
	float3 FinalColor = Energy.A_front * Luminance.xyz * PixelCoverage;

	// Note: No need for pre-exposition, as we fetch & weight SceneColor which is already pre-exposed.

	return -min(-FinalColor, 0.0);
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if LIGHTING_VS

void MainVS(
	in uint InVertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	nointerpolation out uint OutNodeCount : DISPATCH_NODECOUNT,
	nointerpolation out uint2 OutResolution : DISPATCH_RESOLUTION)
{
	OutNodeCount = HairStrands.HairSampleCount[0];
	OutResolution = GetHairSampleResolution(OutNodeCount);

	const float2 ClipCoord = ((OutResolution + 0.5f) / float2(HairStrands.HairSampleViewportResolution)) * 2;

	const float2 UV = float2(InVertexId & 1, InVertexId >> 1);
	const float2 Pos = float2(-UV.x, UV.y) * 4 + float2(-1, +1) + float2(ClipCoord.x, -ClipCoord.y);
	OutPosition = float4(Pos, 0.5f, 1);
}

#endif // LIGHTING_VS

#if LIGHTING_PS

Buffer<uint> IndirectArgsBuffer;

void MainPS(
	float4 SVPos : SV_POSITION,
	uint NodeCount : DISPATCH_NODECOUNT,
	uint2 Resolution : DISPATCH_RESOLUTION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	const uint2 InCoord = uint2(SVPos.xy);
	const uint LocalOffset = InCoord.x + InCoord.y * Resolution.x;
	const uint MaxVisibilityNodeCount = HairStrands.HairSampleCount[0];
	if (LocalOffset >= NodeCount)
	{
		return;
	}

	const uint2 PixelCoord = HairStrands.HairSampleCoords[LocalOffset];
	const float2 SvPosition = PixelCoord + float2(0.5f, 0.5f);

	const float2 UV = (PixelCoord + float2(0.5f, 0.5f)) / float2(View.BufferSizeAndInvSize.xy);
	const float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	const FHairSample Sample = UnpackHairSample(HairStrands.HairSampleData[LocalOffset]);
	const FGBufferData GBuffer = HairSampleToGBufferData(Sample, HairStrands.HairDualScatteringRoughnessOverride);

#if PERMUTATION_DEBUG
	const bool bDebugEnabled = all(int2(PixelCoord) == GetCursorPos());
#else
	const bool bDebugEnabled = false;
#endif

	const bool bScatterSceneLighting = HasHairFlags(Sample.Flags, HAIR_FLAGS_SCATTER_SCENE_LIGHT);
#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SCENECOLOR
	if (bScatterSceneLighting)
#else
	if (!bScatterSceneLighting)
#endif
	{
		const float3 SkyLighting =
		#if PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SCENECOLOR
			ComputeThinTransmission(GBuffer, ScreenPosition, SvPosition, Sample.ControlPointId);
		#elif PERMUTATION_INTEGRATION_TYPE == INTEGRATION_ADHOC
			ReflectionEnvironment(GBuffer, ScreenPosition, SvPosition, Sample.ControlPointId, Sample.MacroGroupId);
		#elif PERMUTATION_INTEGRATION_TYPE == INTEGRATION_UNIFORM
			ReflectionEnvironmentUniformSampling(GBuffer, ScreenPosition, SvPosition, Sample.ControlPointId, Sample.MacroGroupId, bDebugEnabled);
		#elif PERMUTATION_INTEGRATION_TYPE == INTEGRATION_SH
			ReflectionEnvironmentSH(GBuffer, ScreenPosition, SvPosition, Sample.ControlPointId, Sample.MacroGroupId, bDebugEnabled);
		#endif

		const float SampleCoverage = saturate(From8bitCoverage(Sample.Coverage8bit));

		FLightAccumulator LightAccumulator = (FLightAccumulator)0;
		LightAccumulator_Add(LightAccumulator, SkyLighting, SkyLighting, 1.0f, false);
		OutColor.xyz = LightAccumulator_GetResult(LightAccumulator).xyz * SampleCoverage;
		OutColor.a = SampleCoverage;
	}
}

#endif // LIGHTING_PS
