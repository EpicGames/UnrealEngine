// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Roughness remapping functions of each term of the hair BSDF to the default lit model (GGX based)
// This remapping is done based on the hair longitudinal component. This means it does not take into 
// around the anistotropic streatching which happen along the azimuthal plane
float HairToGGXRoughness_R(float R_Roughness)
{
	const float X = saturate(R_Roughness) - 0.51f;
	return clamp(X, 0.1f, 1);
}

float HairToGGXRoughness_TRT(float R_Roughness)
{
	const float X = saturate(R_Roughness);
	const float X2 = X * X;
	const float X3 = X2 * X;
	const float X4 = X2 * X2;

	const float A = -0.207704f;
	const float B = 6.95267f;
	const float C = -10.1783f;
	const float D = 4.24751f;
	const float F = 3.37235f;
	const float G = -1.96947f;
	return (A * X + B * X2 + C * X3 + D * X4) / (G * X + F * X2);
}

float HairToGGXRoughness_TT(float TT_Roughness)
{
	const float X = saturate(TT_Roughness);
	const float X2 = X * X;
	const float X3 = X2 * X;

	const float A = 0.70505f;
	const float B = -0.117104f;
	const float C = 0.0259047f;

	return A * X + B * X2 + C * X3;
}

float4 ComputeRandom4(uint2 PixelPosition, uint SampleIt, uint SampleCount, uint FrameIdMod8)
{
	//  Shader compiler code is pretty good at moving this out of any for loops.
	const uint2 Seed0 = Rand3DPCG16(int3(PixelPosition, FrameIdMod8)).xy;
	const uint2 Seed1 = Rand3DPCG16(int3(PixelPosition + 17, FrameIdMod8)).xy;
	return float4(
		Hammersley16(SampleIt, SampleCount, Seed0),
		Hammersley16(SampleIt, SampleCount, Seed1));
}

// Compute a sample bias by adding a constant bias (3d jitter within a voxel) + a directional offset (jitter along the direction). 
// Both constant & directional bias are scaled with the size of a voxel
float3 ComputePositionBias(float3 SampleDirection, float3 SampleOffset, float VoxelWorldSize, float DepthBiasScale)
{
	const float3 Bias_Constant = SampleOffset;
	const float3 Bias_Directional = SampleDirection * DepthBiasScale;
	return VoxelWorldSize * (Bias_Constant + Bias_Directional);
}

#ifndef USE_HAIR_COMPLEX_TRANSMITTANCE
#define USE_HAIR_COMPLEX_TRANSMITTANCE 0
#endif 

#if USE_HAIR_COMPLEX_TRANSMITTANCE
#include "HairStrandsCommon.ush"
#include "HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrandsDeepTransmittanceDualScattering.ush"
#endif

float3 EvaluateEnvHair(FGBufferData GBuffer, float3 V, float3 N, inout float3 OutL)
{
	const float Shadow = 1;
	const float Backlit = 0;
	const float Area = 0.2f;
	const uint2 Random = uint2(0, 0);

	// Evaluate the hair BSDF for a imaginary reflected direction, and uses it a measure of the directional albedo
	// Use this value to reweight the spherical integration. This is not correct, but give plausible result. 
	// To ensure there is no energy added, clamp the BSDF evaluation to 1.
	OutL = normalize(V - N * dot(V, N));

	FHairTransmittanceData TransmittanceData = InitHairTransmittanceData(true);
#if USE_HAIR_COMPLEX_TRANSMITTANCE
	if (ShouldUseHairComplexTransmittance(GBuffer))
	{
		TransmittanceData = EvaluateDualScattering(GBuffer, V, OutL.xyz);
	}
#endif
	return min(1.f, 2 * PI * HairShading(GBuffer, OutL, V, N, Shadow, TransmittanceData, Backlit, Area, Random));
}