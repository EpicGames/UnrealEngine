// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsBindingCommon.ush"

#if SHADER_FOLLICLE_MASK

float2 OutputResolution;
uint MaxRootCount;
uint MaxUniqueTriangleIndex;
uint Channel;
uint KernelSizeInPixels;

#define UV_MESH 0
#define UV_ROOT 1

#if PERMUTATION_UV_TYPE == UV_MESH
Buffer<float4> UniqueTrianglePositionBuffer;
Buffer<uint>   RootToUniqueTriangleIndexBuffer;
Buffer<uint>   RootBarycentricBuffer; 
#endif 

#if PERMUTATION_UV_TYPE == UV_ROOT
Buffer<float2> RootUVsBuffer;
#endif

void MainVS(
	uint VertexId : SV_VertexID,
	out float2 OutVertexUV : TRI_UVs,
	out float2 OutRootUV : ROOT_UV,
	out float4 OutPosition : SV_POSITION)
{
	OutVertexUV = 0;
	OutRootUV = 0;
	OutPosition = float4(POSITIVE_INFINITY, POSITIVE_INFINITY, POSITIVE_INFINITY, 1);

	ResolvedView = ResolveView();


	const uint RootIndex = VertexId / 3;
	const uint TriangleVertex = (VertexId % 3);

#if PERMUTATION_UV_TYPE == UV_MESH
	uint UniqueTriangleIndex = 0;
	if (RootIndex < MaxRootCount)
	{
		UniqueTriangleIndex = RootToUniqueTriangleIndexBuffer[RootIndex];
	}

	// TODO -> follicle requires UVs

	if (UniqueTriangleIndex > MaxUniqueTriangleIndex) return;
	const FHairMeshBasis Tri = GetHairMeshBasis(UniqueTriangleIndex, UniqueTrianglePositionBuffer, float3(0,0,0));

	const uint EncodedBarycentric = RootBarycentricBuffer[RootIndex];
	const float3 Barycentric = UnpackBarycentrics(EncodedBarycentric);
	float2 RootUV = float2(0, 0);  // TODO Barycentric.x * Tri.UV0 + Barycentric.y * Tri.UV1 + Barycentric.z * Tri.UV2;
#endif

#if PERMUTATION_UV_TYPE == UV_ROOT
	float2 RootUV = RootUVsBuffer[RootIndex];
#endif

	const float2 KernelRadius = float2(KernelSizeInPixels, KernelSizeInPixels);
	const float2 QuadExtent = (KernelRadius / OutputResolution);

	float2 UV = RootUV;
	if (TriangleVertex == 0)
	{
		UV += float2(-QuadExtent.x, -QuadExtent.y);
	}
	if (TriangleVertex == 1)
	{
		UV += float2(3 * QuadExtent.x, -QuadExtent.y);
	}
	if (TriangleVertex == 2)
	{
		UV += float2(-QuadExtent.x, 3 * QuadExtent.y);
	}

	OutRootUV = RootUV;
	OutVertexUV = UV;
	OutPosition = float4(OutVertexUV * 2 - 1, 0.5, 1);
}

void MainPS(
	in float2 VertexUV : TRI_UVs,
	in float2 RootUV : ROOT_UV,
	out float4 OutColor : SV_Target0)
{
	// In a distance of 2 pixels, write a gradient with its pic at the root
	OutColor = 0;
	const float KernelRadius = KernelSizeInPixels;
	const float UVDistance = length(VertexUV - RootUV) * OutputResolution.x;
	const float Value = 1 - saturate(UVDistance / KernelRadius);

	if (Value == 0)
	{
		discard;
	}

	if (Channel==0)
	{
		OutColor = float4(Value, 0, 0, 0);
	}
	else if (Channel==1)
	{
		OutColor = float4(0, Value, 0, 0);
	}
	else if (Channel==2)
	{
		OutColor = float4(0, 0, Value, 0);
	}	
	else if (Channel==3)
	{
		OutColor = float4(0, 0, 0, Value);
	}	
}
#endif // SHADER_FOLLICLE_MASK

#if SHADER_GENERATE_MIPS

uint Resolution;
uint SourceMip;
uint TargetMip;

Texture2D<float4> InTexture;
RWTexture2D<float4> OutTexture;
SamplerState LinearSampler;

struct Gaussian3x3
{
	float Weight[9];
	int Offset;
	int Width;
};

Gaussian3x3 Initialize3x3()
{
	Gaussian3x3 O;
	O.Weight[0] = 1.0; O.Weight[1] = 2.0; O.Weight[2] = 1.0;
	O.Weight[3] = 2.0; O.Weight[4] = 4.0; O.Weight[5] = 2.0;
	O.Weight[6] = 1.0; O.Weight[7] = 2.0; O.Weight[8] = 1.0;

	O.Offset = 1;
	O.Width = 3;

	return O;
}

struct Gaussian5x5
{
	float Weight[25];
	int Offset;
	int Width;
};

Gaussian5x5 Initialize5x5()
{
	Gaussian5x5 O;
	O.Weight[0] = 1;  O.Weight[1] = 4;   O.Weight[2] = 7;   O.Weight[3] = 4;   O.Weight[4] = 1;
	O.Weight[5] = 4;  O.Weight[6] = 16;  O.Weight[7] = 26;  O.Weight[8] = 16;  O.Weight[9] = 4;
	O.Weight[10] = 7; O.Weight[11] = 26; O.Weight[12] = 41; O.Weight[13] = 26; O.Weight[14] = 7;
	O.Weight[15] = 4; O.Weight[16] = 16; O.Weight[17] = 26; O.Weight[18] = 16; O.Weight[19] = 4;
	O.Weight[20] = 1; O.Weight[21] = 4;  O.Weight[22] = 7;  O.Weight[23] = 4;  O.Weight[24] = 1;

	O.Offset = 2;
	O.Width = 5;
	return O;
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{

// Gaussian
#if 1
	//const Gaussian3x3 Kernel = Initialize3x3();
	const Gaussian5x5 Kernel = Initialize5x5();

	const uint2 OutCoord = DispatchThreadId.xy;
	const uint OutResolution = Resolution >> TargetMip;
	if (OutCoord.x >= OutResolution || OutCoord.y >= OutResolution)
		return;

	const float InvResolution = 1.f / float(OutResolution);
	const float KernelDistance = 0.3f;
	float4 Filtered = 0;
	float AccW = 0;

	const float2 CenterUV = (float2(OutCoord)) * InvResolution;
	for (int y = -Kernel.Offset; y <= Kernel.Offset; ++y)
	for (int x = -Kernel.Offset; x <= Kernel.Offset; ++x)
	{
		const uint Index = (Kernel.Offset+x) + (Kernel.Offset+y) * Kernel.Width;
		const float2 UV = CenterUV + float2(x, y) * KernelDistance * InvResolution;
		const float W = Kernel.Weight[Index];
		AccW += W;
		Filtered += W * InTexture.SampleLevel(LinearSampler, UV, 0);
	}

	OutTexture[OutCoord] = Filtered / AccW;
#endif

// Box
#if 0
	if (OutCoord.x >= OutResolution || OutCoord.y >= OutResolution)
		return;

	const uint2 OutCoord = DispatchThreadId.xy;

	const uint2 InCoord0 = uint2(OutCoord.x << 1, OutCoord.y << 1);
	const uint2 InCoord1 = InCoord0 + uint2(1, 0);
	const uint2 InCoord2 = InCoord0 + uint2(0, 1);
	const uint2 InCoord3 = InCoord0 + uint2(1, 1);

	const float4 V0 = InTexture[InCoord0];
	const float4 V1 = InTexture[InCoord1];
	const float4 V2 = InTexture[InCoord2];
	const float4 V3 = InTexture[InCoord3];

	OutTexture[OutCoord] = float4(V0 + V1 + V2 + V3) * 0.25f;
#endif
}

#endif // SHADER_GENERATE_MIPS