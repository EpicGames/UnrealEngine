// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1

#define CULL_ENABLE 0

#define CULLING_CURVE 1
#define CULLING_POINT 2
#define CULLING_TYPE CULLING_CURVE

#if SHADER_RASTERCOMPUTE_CULL
#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

// TODO 
// * HZB
// * Position quantization 3 x 20 bits?
// * Enabling culling (viewport/depth), we need to track of hiddent parts, so that CP type is correct (1:begin/0:middle/2:end)

// Input
Texture2D<float>	SceneDepthTexture;
float2				OutputResolution;
uint				HairMaterialId;
uint				ControlPointCount;
uint				CurveCount;

// Output
RWBuffer<float4>					RWPoints;
RWStructuredBuffer<FPackedHairVis>	RWHairVis;
RWBuffer<uint2>						RWCoord;
RWStructuredBuffer<uint>			RWCounter;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Utils

float3 NDCToPixelCoord(float4 InDC)
{
	const float3 NDC = InDC.xyz / InDC.w;
	float2 UV = NDC.xy * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	return float3(UV * OutputResolution, NDC.z);
}

// Line clipping based on "CLIPPING USING HOMOGENEOUS COORDINATES" by Blinn et al.
bool IsViewportVisible(inout float4 P0, inout float4 P1)
{
	float2 T = float2(0.0f, 1.0f);
	bool bIsRemoved = P0.w < 0.0f && P1.w < 0.0f; // Both points behind near plane

	bool bSign = false;

	UNROLL
	for (uint PlaneIdx = 0; PlaneIdx < 6; ++PlaneIdx)
	{
		// Compute boundary coordinates of both points (w+x, w-x, w+y, w-y, z, w-z)
		bSign = !bSign;
		const uint CompIdx = PlaneIdx / 2;
		const float Sign = bSign ? 1.0f : -1.0f;
		const float WFactor = PlaneIdx != 4 ? 1.0f : 0.0f;
		const float2 BC = WFactor * float2(P0.w, P1.w) + Sign * float2(P0[CompIdx], P1[CompIdx]);

		float Num = BC.x;
		float Denom = BC.x - BC.y;
		bIsRemoved = bIsRemoved || (BC.x < 0.0f && BC.y < 0.0f); // Both points outside the plane
		float Alpha = Num / Denom;

		// If the denominator is negative, P0 has a smaller boundary coordinate than P1, so we can assume
		// that P1 is inside the plane (or bIsRemoved is true), so we need to update the alpha for P0.
		// The reverse is true if the denominator is positive.
		if (Denom < 0.0f)
		{
			T.x = max(T.x, Alpha);
		}
		else
		{
			T.y = min(T.y, Alpha);
		}
	}

	return !bIsRemoved;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Curve version
#if CULLING_TYPE == CULLING_CURVE
groupshared uint s_ValidSegmentCount;
groupshared uint s_WriteOffset;

[numthreads(1024, 1, 1)]
void CSMain(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	if (GroupThreadID == 0)
	{
		s_ValidSegmentCount= 0;
		s_WriteOffset = 0;
	}	
	GroupMemoryBarrierWithGroupSync();

	const uint CurveID = DispatchThreadID;
	const bool bValidCurve = CurveID < CurveCount;
	const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

	bool bIsCurveVisibile = false;
	uint PrimWriteOffset = 0;
	FHairCurve Curve = (FHairCurve)0;
	if (bValidCurve)
	{
		Curve = ReadHairCurve(HairStrandsVF_CurveBuffer, CurveID);
		
		uint VisiblePointCount = 0;
		for (uint PointIt=0; PointIt < Curve.PointCount-1; ++PointIt)
		{
			// Sanity check
			const uint PrimID = PointIt + Curve.PointIndex;
			const bool bValidPoint = PrimID + 1 < ControlPointCount;
			if (bValidPoint)
			{
				// Fetch both control points
				const FHairControlPoint CP0 = ReadHairControlPoint(
					HairStrandsVF_PositionBuffer,
					PrimID,
					PositionOffset,
					HairStrandsVF_Radius,
					HairStrandsVF_RootScale,
					HairStrandsVF_TipScale);

				const FHairControlPoint CP1 = ReadHairControlPoint(
					HairStrandsVF_PositionBuffer,
					PrimID + 1,
					PositionOffset,
					HairStrandsVF_Radius,
					HairStrandsVF_RootScale,
					HairStrandsVF_TipScale);

				// Transform to world space (TODO change this to translated world space?)
				const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
				const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
				float4 SP0 = mul(float4(CP0WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
				float4 SP1 = mul(float4(CP1WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));

				#if CULL_ENABLE
				if (IsViewportVisible(SP0, SP1))
				#endif
				{
					++VisiblePointCount;
					if (CP1.Type == HAIR_CONTROLPOINT_END)
					{
						++VisiblePointCount;
					}
					bIsCurveVisibile = true;
				}
			}
		}
		InterlockedAdd(s_ValidSegmentCount, VisiblePointCount, PrimWriteOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// Alloc
	if (GroupThreadID == 0)
	{
		InterlockedAdd(RWCounter[0], s_ValidSegmentCount, s_WriteOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// Write valid segments
	if (bValidCurve && bIsCurveVisibile)
	{
		uint WriteIndex = s_WriteOffset + PrimWriteOffset;
		for (uint PointIt=0; PointIt < Curve.PointCount; ++PointIt)
		{
			// Sanity check
			const uint PrimID = PointIt + Curve.PointIndex;
			const bool bValidPoint = PrimID < ControlPointCount;
			if (bValidPoint)
			{
				// Fetch both control points
				const FHairControlPoint CP0 = ReadHairControlPoint(
					HairStrandsVF_PositionBuffer,
					PrimID,
					PositionOffset,
					HairStrandsVF_Radius,
					HairStrandsVF_RootScale,
					HairStrandsVF_TipScale);

				const FHairControlPoint CP1 = ReadHairControlPoint(
					HairStrandsVF_PositionBuffer,
					PrimID + 1,
					PositionOffset,
					HairStrandsVF_Radius,
					HairStrandsVF_RootScale,
					HairStrandsVF_TipScale);

				// Transform to world space (TODO change this to translated world space?)
				const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
				const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
				float4 SP0 = mul(float4(CP0WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
				float4 SP1 = mul(float4(CP1WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));

				#if CULL_ENABLE
				if (IsViewportVisible(SP0, SP1))
				#endif
				{
					SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
					SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);

					FHairControlPoint Point0 = CP0;
					Point0.Position = CP0WP;

					const float DeviceZ = SP0.z;
					const float2 Coord = SP0.xy;

					FHairVis Out;
					Out.Depth			= DeviceZ;
					Out.Coverage		= 1;
					Out.Coverage8bit	= 255;
					Out.ControlPointId		= PrimID;
					Out.MaterialId		= HairMaterialId;

					uint PackedW = f32tof16(Point0.WorldRadius);
					PackedW = PackedW | (uint(saturate(Point0.UCoord) * 0xFF)<<16);
					PackedW = PackedW | (Point0.Type<<24);

					RWHairVis[WriteIndex] = PackHairVis(Out);
					RWCoord[WriteIndex]   = Coord;
					RWPoints[WriteIndex] = float4(Point0.Position, asfloat(PackedW));
					++WriteIndex;
				}
			}
		}
	}
}
#endif // CULLING_TYPE == CULLING_CURVE

///////////////////////////////////////////////////////////////////////////////////////////////////
// Point version
#if CULLING_TYPE == CULLING_POINT
groupshared uint s_ValidSegmentCount;
groupshared uint s_WriteOffset;

[numthreads(1024, 1, 1)]
void CSMain(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	if (GroupThreadID == 0)
	{
		s_ValidSegmentCount= 0;
		s_WriteOffset = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	uint   ControlPointId = DispatchThreadID;
	bool   bIsPrimVisibile = false;
	uint   PrimWriteOffset = 0;
	float  DeviceZ = 0;
	float2 Coord = 0;

	FHairControlPoint Point0 = (FHairControlPoint)0;

	// Count valid segments
	bool bIsPrimValid = ControlPointId < ControlPointCount;
	if (bIsPrimValid)
	{
		const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

		// Fetch both control points
		FHairControlPoint CP0 = ReadHairControlPoint(
			HairStrandsVF_PositionBuffer,
			ControlPointId,
			PositionOffset,
			HairStrandsVF_Radius,
			HairStrandsVF_RootScale,
			HairStrandsVF_TipScale);

		FHairControlPoint CP1 = CP0;
		if (CP0.Type != HAIR_CONTROLPOINT_END)
		{
			CP1 = ReadHairControlPoint(
				HairStrandsVF_PositionBuffer,
				ControlPointId + 1,
				PositionOffset,
				HairStrandsVF_Radius,
				HairStrandsVF_RootScale,
				HairStrandsVF_TipScale);
		}

		// Transform to world space
		const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
		const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
		float4 SP0 = mul(float4(CP0WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
		float4 SP1 = mul(float4(CP1WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));

		#if CULL_ENABLE
		if (IsViewportVisible(SP0, SP1))
		#endif
		{
			SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
			SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);

			Point0 = CP0;
			Point0.Position = CP0WP;

			DeviceZ = SP0.z;
			Coord = SP0.xy;
			bIsPrimVisibile = true;
			InterlockedAdd(s_ValidSegmentCount, 1, PrimWriteOffset);
		}
	}	
	GroupMemoryBarrierWithGroupSync();

	// Alloc
	if (GroupThreadID == 0)
	{
		InterlockedAdd(RWCounter[0], s_ValidSegmentCount, s_WriteOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// Write valid segments
	if (bIsPrimVisibile)
	{
		FHairVis Out;
		Out.Depth			= DeviceZ;
		Out.Coverage		= 1;
		Out.Coverage8bit	= 255;
		Out.ControlPointId	= ControlPointId;
		Out.MaterialId		= HairMaterialId;

		uint PackedW = f32tof16(Point0.WorldRadius);
		PackedW = PackedW | (uint(saturate(Point0.UCoord) * 0xFF)<<16);
		PackedW = PackedW | (Point0.Type<<24);

		// TODO: for now, no culling, at all, write all segments
		const uint WriteIndex = s_WriteOffset + PrimWriteOffset;
		//const uint WriteIndex = PrimID;
		RWHairVis[WriteIndex] = PackHairVis(Out);
		RWCoord[WriteIndex]   = Coord;
		RWPoints[WriteIndex] = float4(Point0.Position, asfloat(PackedW));
	}
}
#endif // CULLING_TYPE == CULLING_CURVE

#endif // SHADER_RASTERCOMPUTE_CULL