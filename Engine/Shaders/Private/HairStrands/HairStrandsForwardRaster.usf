// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 0

#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "../ColorMap.ush"

#if PERMUTATION_DEBUG
#include "../ShaderPrint.ush"
#endif

////////////////////////////////////////////////////////////
// Pack/unpack helpers
uint PackTileCoord(uint2 In)
{
	return (In.x & 0xffff) | ((In.y & 0xffff) << 16);
}

uint2 UnpackTileCoord(uint In)
{
	return uint2(In & 0xffff, (In >> 16) & 0xffff);
}

uint PackDepth(float In)
{
	return asuint(In);
}

float UnpackDepth(uint In)
{
	return asfloat(In);
}

struct FDepthRange
{
	float MinZ;
	float MaxZ;
};

uint PackDepthRange(FDepthRange In)
{
	return PackFloat2ToUInt(In.MinZ, In.MaxZ);
}

FDepthRange UnpackDepthRange(uint In)
{
	FDepthRange Out;
	const float2 D = UnpackFloat2FromUInt(In);
	Out.MinZ = D.x;
	Out.MaxZ = D.y;
	return Out;
}

uint PackWork(uint InTileIndex, uint InTileCount)
{
	return InTileIndex | (InTileCount << 16);
}

uint2 UnpackWork(uint In)
{
	return uint2(In & 0xFFFF, In >> 16);
}

///////////////////////////////////////////////////////////////////////////
// Tile Helpers

// Max number of iterator that a rasterizer can do. This is for preventing any kind of infinite loop.
#define MAX_WORK_COUNT 4096

#define BIN_TILE_SIZE 32
#define BIN_RCP_TILE_SIZE (1.f / BIN_TILE_SIZE)
#define BIN_TILE_SIZE_AS_SHIFT 5
#define BIN_THREAD_COUNT 1024

#define RASTER_TILE_SIZE 8
#define RASTER_RCP_TILE_SIZE (1.f / RASTER_TILE_SIZE)
#define RASTER_TILE_SIZE_AS_SHIFT 3
#define RASTER_THREAD_COUNT 64

uint2 LinearTo2D_Common(uint In, uint InTileSize, float InRcpTileSize)
{
	uint2 Out;
#if 0
	Out.y = (In + 0.5f) * InRcpTileSize;
	Out.x = In - (Out.y * InTileSize);
#else
	Out.x = In%InTileSize;
	Out.y = In/InTileSize;
#endif
	return Out;
}

uint2 LinearTo2D_Bin(uint In)	 { return LinearTo2D_Common(In, BIN_TILE_SIZE, BIN_RCP_TILE_SIZE); }
uint2 LinearTo2D_Raster(uint In) { return LinearTo2D_Common(In, RASTER_TILE_SIZE, RASTER_RCP_TILE_SIZE); }

///////////////////////////////////////////////////////////////////////////
// DDA helper
 
// TODO: Without setting a limit, we are sometimes caught in an infinite loop even though this should not happen.
#define DDA_MAX_ITERATIONS 256

struct FDDAContext
{
	float2 Coord;
	float2 DeltaDist;
	float2 Step;
	float2 SideDist;
};

FDDAContext DDACreateContext(float2 RayStart, float2 RayDir)
{
	const float2 RayDirRcp = 1.0f / RayDir;

	FDDAContext Context;
	Context.Coord = floor(RayStart);
	Context.DeltaDist = abs(RayDirRcp);
	Context.Step = sign(RayDir);
	Context.SideDist = (Context.Coord - RayStart + max(Context.Step, 0.0f)) * RayDirRcp;

	return Context;
}

void DDAAdvance(inout FDDAContext Context)
{
	if (Context.SideDist.x < Context.SideDist.y)
	{
		Context.SideDist.x += Context.DeltaDist.x;
		Context.Coord.x += Context.Step.x;
	}
	else
	{
		Context.SideDist.y += Context.DeltaDist.y;
		Context.Coord.y += Context.Step.y;
	}
}

///////////////////////////////////////////////////////////////////////////
// Visibility Tile data
/*
// use untyped buffer for segment tiles to reduce VGPR usage - 16 bytes
struct FVisTile
{
	uint PrimOffset;
	uint PrimCount;
	uint TileCoord;
	uint MinDepth;
};
*/
#define VT_PrimOffset 0
#define VT_PrimCount 1
#define VT_Coord 2
#define VT_MinWriteIndex 3
#define VT_MinMaxDepth 4
#define VT_SIZE 5

// Visibility tile data are stored as:
//   ________________________________________________________________________________________________________________________________________________________________________________________________________________
// ||                      Tile 0                                        ||                      Tile 1                                        ||                      Tile 2                                        ||
// ||____________________________________________________________________||____________________________________________________________________||____________________________________________________________________||
// ||            |           |             |               |             ||            |           |             |               |             ||            |           |             |               |             ||
// || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex | MinMaxDepth || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex | MinMaxDepth || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex | MinMaxDepth ||

uint PackVisTileCoord(uint2 In)
{
	return (In.x & 0xff) | ((In.y & 0xff) << 8);
}

uint2 UnpackVisTileCoord(uint In)
{
	return uint2(In & 0xff, (In >> 8) & 0xff);
}

uint LoadOutVisTileData(RWByteAddressBuffer OutBuffer, uint Index, uint VTEntry)
{
	// Each entry is 4 bytes
	return OutBuffer.Load(Index * VT_SIZE * 4 + VTEntry * 4);
}

void StoreOutVisTileData(RWByteAddressBuffer OutBuffer, uint Index, uint VTEntry, uint Value)
{
	// Each entry is 4 bytes
	OutBuffer.Store(Index * VT_SIZE * 4 + VTEntry * 4, Value);
}

uint LoadVisTileData(ByteAddressBuffer InBuffer, uint Index, uint VTEntry)
{
	return InBuffer.Load(Index * VT_SIZE * 4 + VTEntry * 4);
}

///////////////////////////////////////////////////////////////////////////
// Misc.
float4 Transparent(float4 Color) { return float4(Color.xyz, 0.5f); }

///////////////////////////////////////////////////////////////////////////
// Common parameters

int2	BinTileRes;
int2	RasterTileRes;

uint	NumBinners;
float	RcpNumBinners;

uint	NumRasterizers;
float	RcpNumRasterizers;

int2	OutputResolution;
float2	OutputResolutionf;
float	RadiusAtDepth1;

///////////////////////////////////////////////////////////////////////////
// Control points
Buffer<float4> ControlPoints;
StructuredBuffer<uint> ControlPointCount;
uint MaxControlPointCount;

uint GetControlPointCount()
{
	return ControlPointCount[0];
}

// Custom encoding for forward rasterizer. Position are 32bits float. This is temporary.
FHairControlPoint UnpackHairControlPoint(uint InPrimId)
{
	const float4 Packed = ControlPoints[InPrimId];
	const uint W = asuint(Packed.w);
	const float R = f16tof32(W & 0xFFFF);
	const float U = ((W >> 16) & 0xFF) * (1.f / 255.f);
	const uint  T = (W >> 24) & 0x3;

	FHairControlPoint Out;
	Out.Position = Packed.xyz;
	Out.WorldRadius = R;
	Out.UCoord = U;
	Out.Type = T;
	return Out;
}

#if SHADER_RASTERCOMPUTE_BINNING || SHADER_RASTERCOMPUTE_COMPACTION || SHADER_RASTERCOMPUTE_RASTER || SHADER_RASTERCOMPUTE_DEPTH_GRID || SHADER_RASTERCOMPUTE_DEBUG

///////////////////////////////////////////////////////////////////////////

float3 NDCToPixelCoord(float4 InDC)
{
	const float3 NDC = InDC.xyz / InDC.w;
	float2 UV = NDC.xy * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	return float3(UV * OutputResolution, NDC.z);
}

void CalcHomogenousPos(in uint PrimId, out float4 HP, out uint Type)
{
	const FHairControlPoint CP = UnpackHairControlPoint(PrimId);

	const float3 WP = CP.Position; // This is actually WorldPosition
	HP = mul(float4(WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip)); // TODO move this at least into translated world space
	Type = CP.Type;
}

void CalcHomogenousPosAndRad(in uint PrimId, out float4 HP, out float Rad, out uint Type)
{
	const FHairControlPoint CP = UnpackHairControlPoint(PrimId);

	const float3 WP = CP.Position; // This is actually WorldPosition
	HP = mul(float4(WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
	Rad = CP.WorldRadius;
	Type = CP.Type;
}

float ComputeLerpAlpha(int2 Coord, float2 P0, float2 P1, float SegmentLenSqRcp)
{
	// Project P onto line segment and compute the lerp alpha between P0 and P1
	// Simplification of:
	// A = P - P0
	// B = P1 - P0
	// Alpha = dot(A, B) / dot(B, B)
	const float2 P = Coord + 0.5f;
	const float Alpha = saturate(dot(P - P0, P1 - P0) * SegmentLenSqRcp);
	return Alpha;
}

float ComputePerspectiveCorrectRadius(float Rad0, float Rad1, float Alpha, float RcpW0, float RcpW1)
{
	// Alpha value for perspective correct interpolation. We store the reciprocal of w in the w component of P0 and P1,
	// so this is a simplification of:
	// (Alpha / w1) / ((1 - Alpha) / w0 + Alpha / w1)
	const float LerpedRcpW = lerp(RcpW0, RcpW1, Alpha);
	const float PerspectiveAlpha = (Alpha * RcpW1) / LerpedRcpW;
	// Divide by W to make thickness dependent on screen space depth? This division was kept from the previous line rasterization algorithm.
	const float Rad = lerp(Rad0, Rad1, PerspectiveAlpha) * LerpedRcpW;
	return Rad;
}

// Line clipping based on "CLIPPING USING HOMOGENEOUS COORDINATES" by Blinn et al.
bool BlinnLineClipping(inout float4 P0, inout float4 P1)
{
	float2 T = float2(0.0f, 1.0f);
	bool bIsRemoved = P0.w < 0.0f && P1.w < 0.0f; // Both points behind near plane

	bool bSign = false;

	UNROLL
	for (uint PlaneIdx = 0; PlaneIdx < 6; ++PlaneIdx)
	{
		// Compute boundary coordinates of both points (w+x, w-x, w+y, w-y, z, w-z)
		bSign = !bSign;
		const uint CompIdx = PlaneIdx / 2;
		const float Sign = bSign ? 1.0f : -1.0f;
		const float WFactor = PlaneIdx != 4 ? 1.0f : 0.0f;
		const float2 BC = WFactor * float2(P0.w, P1.w) + Sign * float2(P0[CompIdx], P1[CompIdx]);

		float Num = BC.x;
		float Denom = BC.x - BC.y;
		bIsRemoved = bIsRemoved || (BC.x < 0.0f && BC.y < 0.0f); // Both points outside the plane
		float Alpha = Num / Denom;
		
		// If the denominator is negative, P0 has a smaller boundary coordinate than P1, so we can assume
		// that P1 is inside the plane (or bIsRemoved is true), so we need to update the alpha for P0.
		// The reverse is true if the denominator is positive.
		if (Denom < 0.0f)
		{
			T.x = max(T.x, Alpha);
		}
		else
		{
			T.y = min(T.y, Alpha);
		}
	}

	if (!bIsRemoved)
	{
		const float4 P0Clipped = lerp(P0, P1, T.x);
		const float4 P1Clipped = lerp(P0, P1, T.y);
		P0 = P0Clipped;
		P1 = P1Clipped;
	}

	return !bIsRemoved;
}

bool ClipRaySegment(float2 AABBMin, float2 AABBMax, float4 P0, float4 P1, out float2 T, out bool2 bClipped)
{
	bClipped = false;
	T = float2(0.0f, 1.0f);
	const bool bP0Outside = any(P0.xy < AABBMin) || any(P0.xy > AABBMax);
	const bool bP1Outside = any(P1.xy < AABBMin) || any(P1.xy > AABBMax);
	if (!bP0Outside && !bP1Outside)
	{
		return true;
	}

	const float2 Origin = P0.xy;
	const float2 Dir = P1.xy - P0.xy;
	const float2 RcpDir = 1.0f / Dir;

	const float2 T0 = (AABBMin - Origin) * RcpDir;
	const float2 T1 = (AABBMax - Origin) * RcpDir;

	T.x = max(min(T0.x, T1.x), min(T0.y, T1.y));
	T.y = min(max(T0.x, T1.x), max(T0.y, T1.y));

	// Ray intersects the AABB but the segment is completely outside or no intersection at all.
	if (T.y < 0.0f || T.x > T.y || T.x > 1.f)
	{
		bClipped = true;
		return false;
	}

	if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
	{
		bClipped.x = true;
	}
	if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
	{
		bClipped.y = true;
	}

	return true;
}

bool ClipRaySegment(float2 AABBMin, float2 AABBMax, inout float4 P0, inout float4 P1, inout float Rad0, inout float Rad1, inout float Alpha0, inout float Alpha1, out bool2 bClipped, out float2 T)
{
	//float2 T;
	bool bIsValid = ClipRaySegment(AABBMin, AABBMax, P0, P1, T, bClipped);
	
	if (bIsValid)
	{
		const bool bP0Outside = any(P0.xy < AABBMin) || any(P0.xy > AABBMax);
		const bool bP1Outside = any(P1.xy < AABBMin) || any(P1.xy > AABBMax);

		float4 P0New = P0;
		float4 P1New = P1;
		float Rad0New = Rad0;
		float Rad1New = Rad1;
		if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
		{
			Alpha0 = T.x;
			P0New = lerp(P0, P1, T.x);
			Rad0New = lerp(Rad0, Rad1, T.x);
			bClipped.x = true;
		}
		if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
		{
			Alpha1 = T.y;
			P1New = lerp(P0, P1, T.y);
			Rad1New = lerp(Rad0, Rad1, T.y);
			bClipped.y = true;
		}
		P0 = P0New;
		P1 = P1New;
		Rad0 = Rad0New;
		Rad1 = Rad1New;
	}

	return bIsValid;
}

#endif // Common rasetrizer helper function & parameters

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_DEPTH_GRID

Texture2D<float> SceneDepthTexture;
RWTexture2D<uint> OutVisTileDepthGrid;
groupshared uint group_FurthestDepth; // (4 bytes)

[numthreads(BIN_THREAD_COUNT, 1, 1)]
void PrepareDepthGridCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	if (GroupThreadID == 0)
	{
		group_FurthestDepth = 0xFFFFFFFF; // Inverse-Z
	}

	GroupMemoryBarrierWithGroupSync();

	// Assumes a workgroup size of 1024 threads to cover a maximum tile size of 32x32.
	if (GroupThreadID < BIN_THREAD_COUNT)
	{
		const uint2 PixelCoord = LinearTo2D_Bin(GroupThreadID) + GroupID * BIN_TILE_SIZE;

		if (all(PixelCoord < (uint2)OutputResolution))
		{
			const float Depth = SceneDepthTexture.Load(uint3(PixelCoord, 0));

			// Compute furthest depth inside this tile
			WaveInterlockedMin(group_FurthestDepth, PackDepth(Depth)); // Inverse-Z
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadID == 0)
	{
		OutVisTileDepthGrid[GroupID] = group_FurthestDepth;
	}
}

#endif //SHADER_RASTERCOMPUTE_DEPTH_GRID

///////////////////////////////////////////////////////////////////////////

#define BIN_MINMAX 1

#if SHADER_RASTERCOMPUTE_BINNING

RWTexture2DArray<uint> 				OutVisTileBinningGrid;
#if BIN_MINMAX
RWTexture2DArray<uint> 				OutVisTileBinningGridMinZ;
RWTexture2DArray<uint> 				OutVisTileBinningGridMaxZ;
#endif
RWBuffer<uint>						OutVisTilePrims;
RWBuffer<uint>						OutVisTilePrimDepths;
RWBuffer<uint>						OutVisTileArgs;
RWByteAddressBuffer					OutVisTileData;
Texture2D<uint>						VisTileDepthGrid;
ByteAddressBuffer					IndirectPrimIDCount;

groupshared uint group_LoopNum;
groupshared uint group_VerticesNum;
groupshared uint group_BatchNum;

#define TILES_TO_ALLOCATE_MAX 1024

groupshared uint group_TilesToAllocate[TILES_TO_ALLOCATE_MAX];
groupshared uint group_TilesToAllocateCount;

// The total number of line segments (ControlPointCount) is divided up equally between N binners - each binner = a workgroup which loops through the designated set segments in batches of 1024
// NB there is still potential to use LDS to prevent/reduce contention when adding segments to the binning grid and this may improve perf

[numthreads(1024, 1, 1)]
void BinningCS(uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();
	if (GroupThreadID == 0)
	{
		group_VerticesNum = GetControlPointCount();
		group_BatchNum = DivideAndRoundUp(group_VerticesNum, 1024);
		group_LoopNum  = DivideAndRoundUp(group_BatchNum, NumBinners);
	}
	GroupMemoryBarrierWithGroupSync();

	#if PERMUTATION_DEBUG
	const bool bDebugEnabled = false && GroupID == 0 && GroupThreadID <= 64;
	FShaderPrintContext Ctx = InitShaderPrintContext(bDebugEnabled, uint2(250, 50));
	#endif

	LOOP for (uint LoopIndex = 0; LoopIndex < group_LoopNum; LoopIndex++)
	{
		const uint BatchIndex = LoopIndex + (GroupID * group_LoopNum);
		bool bSegValid = (BatchIndex < group_BatchNum);

		const uint PrimID = BatchIndex * 1024 + GroupThreadID;
		bSegValid = bSegValid && (PrimID < group_VerticesNum);

		const uint SegmentCountLayerIdx = GroupID; // Stores number of segments per tile per workgroup.
		const uint TmpSegmentCountLayerIdx = SegmentCountLayerIdx + NumBinners; // Also stores number of segments per tile per workgroup. Used as second counter for this two pass algorithm.
		const uint TileAllocInfoLayerIdx = SegmentCountLayerIdx + NumBinners * 2; // Stores per tile per workgroup allocation info.

		uint MaxZ = 0;
		uint MinZ = 0xFFFFFFFF;
		float2 TileCoord0F = 0.0f;
		float2 TileCoord1F = 0.0f;

		#if PERMUTATION_DEBUG
		FHairControlPoint CP0;
		FHairControlPoint CP1;
		#endif

		// 1. Project segment end points and clip them to the screen
		if (bSegValid)
		{
			float4 H0 = 0.0f;
			float4 H1 = 0.0f;
			uint Type = -1;
			CalcHomogenousPos(PrimID, H0, Type);

			bool bIsEndCV = (Type == HAIR_CONTROLPOINT_END);
			bSegValid = !bIsEndCV;

			if (bSegValid)
			{
				CalcHomogenousPos(PrimID + 1, H1, Type);
				
				// Do clipping in homogenous coordinates
				bSegValid = BlinnLineClipping(H0, H1);

				if (bSegValid)
				{
					float3 SP0 = NDCToPixelCoord(H0);
					float3 SP1 = NDCToPixelCoord(H1);
					SP0.xy *= BIN_RCP_TILE_SIZE;
					SP1.xy *= BIN_RCP_TILE_SIZE;

					// For peace of mind, make sure these are actually clamped to a valid range.
					SP0 = clamp(SP0, 0.0f, float3(BinTileRes-0.01f, 1.0f));
					SP1 = clamp(SP1, 0.0f, float3(BinTileRes-0.01f, 1.0f));

					MaxZ = PackDepth(max(SP0.z, SP1.z));
					MinZ = PackDepth(min(SP0.z, SP1.z));

					TileCoord0F = SP0.xy;
					TileCoord1F = SP1.xy;

					#if PERMUTATION_DEBUG
					if (bDebugEnabled && 0)
					{
						CP0 = UnpackHairControlPoint(PrimID);
						CP1 = UnpackHairControlPoint(PrimID +1);

						AddLineWS(Ctx, CP0.Position, CP1.Position, ColorRed);
					}
					#endif
				}
			}
		}

		// 2. Reset allocation counter
		if (GroupThreadID == 0)
		{
			group_TilesToAllocateCount = 0;
		}
		GroupMemoryBarrierWithGroupSync();
		
		// 3. Increment per workgroup per tile counters and add tiles to be allocated
		if (bSegValid)
		{
			FDDAContext DDAContext = DDACreateContext(TileCoord0F, normalize(TileCoord1F - TileCoord0F));
			const int2 EndCoord = (int2)floor(TileCoord1F);

			for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
			{
				const int2 TileCoord = (int2)floor(DDAContext.Coord);
				
				uint DebugInsertMode = 0;
				BRANCH
				if (MaxZ > VisTileDepthGrid[TileCoord])  // Inverse-Z
				{
					uint OldTileSegmentCount;
					InterlockedAdd(OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)], 1, OldTileSegmentCount);
					DebugInsertMode = 1;

					// Min/Max
					#if BIN_MINMAX
					InterlockedMin(OutVisTileBinningGridMinZ[uint3(TileCoord, SegmentCountLayerIdx)], MinZ);
					InterlockedMax(OutVisTileBinningGridMaxZ[uint3(TileCoord, SegmentCountLayerIdx)], MaxZ);
					#endif

					BRANCH
					if ((OldTileSegmentCount % 1024) == 0)
					{
						uint WritePos;
						InterlockedAdd(group_TilesToAllocateCount, 1, WritePos);
						if (WritePos < TILES_TO_ALLOCATE_MAX)
						{
							group_TilesToAllocate[WritePos] = PackVisTileCoord(TileCoord);
							DebugInsertMode = 2;
						}
					}
				}

				#if PERMUTATION_DEBUG
				if (bDebugEnabled)
				{
					//CP0 = UnpackHairControlPoint(PrimID);
					//CP1 = UnpackHairControlPoint(PrimID +1);
					//AddLineWS(Ctx, CP0.Position, CP1.Position, ColorRed);

					float4 DebugColor = ColorRed;
					if (DebugInsertMode == 1) DebugColor = ColorGreen;
					if (DebugInsertMode == 2) DebugColor = ColorYellow;
					AddQuadSS(Ctx, TileCoord * BIN_TILE_SIZE, TileCoord * BIN_TILE_SIZE + BIN_TILE_SIZE, DebugColor);
				}
				#endif

				if (all(TileCoord == EndCoord))
				{
					break;
				}

				DDAAdvance(DDAContext);
			}
		}

		GroupMemoryBarrierWithGroupSync();

		// 4. Allocate tiles
		const uint TilesToAllocateCount = min(TILES_TO_ALLOCATE_MAX, group_TilesToAllocateCount);
		for (uint TileIdx = GroupThreadID; TileIdx < TilesToAllocateCount; TileIdx += 1024)
		{
			const uint PackedTileCoord = group_TilesToAllocate[TileIdx];
			const uint2 TileCoord = UnpackVisTileCoord(PackedTileCoord);

			const uint TotalNewWriteCount = OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)];
			const uint TotalOldWriteCount = OutVisTileBinningGrid[uint3(TileCoord, TmpSegmentCountLayerIdx)];

			#if BIN_MINMAX
			FDepthRange TileDepthRange;
			TileDepthRange.MinZ = UnpackDepth(OutVisTileBinningGridMinZ[uint3(TileCoord, SegmentCountLayerIdx)]);
			TileDepthRange.MaxZ = UnpackDepth(OutVisTileBinningGridMaxZ[uint3(TileCoord, SegmentCountLayerIdx)]);
			#endif

			uint NewTile;
			WaveInterlockedAddScalar_(OutVisTileArgs[0], 1, NewTile);

			StoreOutVisTileData(OutVisTileData, NewTile, VT_Coord, PackedTileCoord);
			// Round down the count to the start of the tile and later compare against this to decide which tile to write to.
			StoreOutVisTileData(OutVisTileData, NewTile, VT_MinWriteIndex, TotalNewWriteCount & ~1023u);
			// Min/Max depth
			#if BIN_MINMAX
			StoreOutVisTileData(OutVisTileData, NewTile, VT_MinMaxDepth, PackDepthRange(TileDepthRange));
			#endif

			const uint PrevTile = (OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)] & 0xffff);

			if (TotalOldWriteCount > 0)
			{
				StoreOutVisTileData(OutVisTileData, PrevTile, VT_PrimCount, 1024);
			}

			OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)] = (PrevTile << 16) | (NewTile & 0xffff);
		}

		GroupMemoryBarrierWithGroupSync();

		// 5. Write PrimID to tiles
		if (bSegValid)
		{
			FDDAContext DDAContext = DDACreateContext(TileCoord0F, normalize(TileCoord1F - TileCoord0F));
			const int2 EndCoord = (int2)floor(TileCoord1F);

			for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
			{
				const int2 TileCoord = (int2)floor(DDAContext.Coord);

				BRANCH
				if (MaxZ > VisTileDepthGrid[TileCoord])  // Inverse-Z
				{
					const uint PackedTiles = OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)];
					const uint CurTile = (PackedTiles & 0xffff);
					const uint PrevTile = ((PackedTiles >> 16) & 0xffff);

					// Currently we need this to get our write position, but maybe there is a cheaper way to keep track of that?
					uint OldTileSegmentCount;
					InterlockedAdd(OutVisTileBinningGrid[uint3(TileCoord, TmpSegmentCountLayerIdx)], 1, OldTileSegmentCount);

					const bool bWriteToCurTile = OldTileSegmentCount >= LoadOutVisTileData(OutVisTileData, CurTile, VT_MinWriteIndex);
					const uint LocalWritePos = OldTileSegmentCount % 1024;
					const uint WritePos = (bWriteToCurTile ? CurTile : PrevTile) * 1024 + LocalWritePos;

					OutVisTilePrims[WritePos] = PrimID;
					OutVisTilePrimDepths[WritePos] = MaxZ; // Inverse-Z
					BRANCH
					if (bWriteToCurTile)
					{
						if ((OldTileSegmentCount + 1) == OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)])
						{
							StoreOutVisTileData(OutVisTileData, CurTile, VT_PrimCount, (OldTileSegmentCount == 1023) ? 1024 : ((OldTileSegmentCount + 1) % 1024));
						}
					}
				}

				if (all(TileCoord == EndCoord))
				{
					break;
				}

				DDAAdvance(DDAContext);
			}
		}
	}
}
#endif //SHADER_RASTERCOMPUTE_BINNING

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_COMPACTION

ByteAddressBuffer					InData;
Buffer<uint> 						InPrims;
Buffer<uint> 						InDepths;
Buffer<uint> 						InArgs;
RWByteAddressBuffer					OutData;
RWBuffer<uint> 						OutPrims;
RWBuffer<uint> 						OutArgs;
RWStructuredBuffer<uint> 			OutWork; // Offset & Count
RWStructuredBuffer<uint> 			OutDataCount;
RWStructuredBuffer<uint> 			OutWorkCount;

groupshared uint group_TotalPrimCount;
groupshared uint group_PrimWriteOffset;
groupshared uint group_NumTiles;
groupshared uint group_TilesToCompact[1024];
groupshared uint group_MaxLDSTileIdx;
groupshared uint group_MinZ;
groupshared uint group_MaxZ;

#define COMPACTION_DEPTH_BUCKET 1024
groupshared uint s_BinOffset[COMPACTION_DEPTH_BUCKET];
groupshared uint s_BinCount[COMPACTION_DEPTH_BUCKET];

uint GetDepthBinIndex(float InDepth)
{
	// Inverse-Z
	const float MinDepth = UnpackDepth(group_MinZ);
	const float MaxDepth = UnpackDepth(group_MaxZ);
	const float InvDepthExtent = 1.f / max(MaxDepth - MinDepth, 1e-5f);
	const uint DepthIt = clamp(saturate((InDepth - MinDepth) * InvDepthExtent) * COMPACTION_DEPTH_BUCKET, 0, COMPACTION_DEPTH_BUCKET - 1);
	return (COMPACTION_DEPTH_BUCKET - 1) - DepthIt;
}

// Launch based on CPU BinTileResX x BinTileResY
// 1 group per screen-tile, 1 threads per bin-tile matching the screen-tile coord
// There can be/are several bins for the same screen area
[numthreads(1024, 1, 1)]
void CompactionCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	if (GroupThreadID == 0)
	{
		group_TotalPrimCount = 0;
		group_NumTiles = 0;
		group_MaxLDSTileIdx = 0;
		group_MinZ = 0xFFFFFFFF;
		group_MaxZ = 0;
	}
	if (GroupThreadID < COMPACTION_DEPTH_BUCKET)
	{
		s_BinCount[GroupThreadID] = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	const uint NumTiles = InArgs[0];
	const uint PackedCoord = PackVisTileCoord(GroupID); // All thread will process the same tile

	#if PERMUTATION_DEBUG
	const uint2 TileCoord = UnpackVisTileCoord(PackedCoord);
	const bool bDebugEnabled = false && all(TileCoord == uint2(ShaderPrintData.CursorCoord / float(BIN_TILE_SIZE)));
	FShaderPrintContext Ctx = InitShaderPrintContext(bDebugEnabled, uint2(750, 50));
	#endif

	// 1. Compute total number of primitives at this tile coordinate
	uint LocalPrimCount = 0;
	{
		for (uint TileIdx = GroupThreadID; TileIdx < NumTiles; TileIdx += 1024)
		{
			const uint TilePackedCoord = LoadVisTileData(InData, TileIdx, VT_Coord);
			if (PackedCoord == TilePackedCoord)
			{
				LocalPrimCount += LoadVisTileData(InData, TileIdx, VT_PrimCount);

				const FDepthRange LocalDepthRange = UnpackDepthRange(LoadVisTileData(InData, TileIdx, VT_MinMaxDepth));
				InterlockedMin(group_MinZ, PackDepth(LocalDepthRange.MinZ));
				InterlockedMax(group_MaxZ, PackDepth(LocalDepthRange.MaxZ));
				
				uint WritePos;
				WaveInterlockedAddScalar_(group_NumTiles, 1, WritePos);
				if (WritePos < 1024)
				{
					group_TilesToCompact[WritePos] = TileIdx;
					WaveInterlockedMax(group_MaxLDSTileIdx, TileIdx);
				}
			}
		}
	}
	GroupMemoryBarrierWithGroupSync();

	if (LocalPrimCount > 0)
	{
		WaveInterlockedAdd(group_TotalPrimCount, LocalPrimCount);
	}
	GroupMemoryBarrierWithGroupSync();

	const uint TotalPrimCount = group_TotalPrimCount;

	if (TotalPrimCount == 0)
	{
		return;
	}

	// 2. Allocate space
	if (GroupThreadID == 0)
	{
		uint NumTilesToAllocate = DivideAndRoundUp(TotalPrimCount, 1024);

		uint FirstCompactedTile;
		InterlockedAdd(OutArgs[0], NumTilesToAllocate, FirstCompactedTile);

		uint WorkIndex;
		InterlockedAdd(OutWorkCount[0], 1, WorkIndex);
		OutWork[WorkIndex] = PackWork(FirstCompactedTile, NumTilesToAllocate);

		group_PrimWriteOffset = FirstCompactedTile * 1024;

		// Initialize new tiles
		for (uint TileIdx = 0; TileIdx < NumTilesToAllocate; ++TileIdx)
		{
			const uint CompactedTile = FirstCompactedTile + TileIdx;

			const uint PrimCount = min(TotalPrimCount - TileIdx * 1024, 1024);
			StoreOutVisTileData(OutData, CompactedTile, VT_PrimCount, PrimCount);
			StoreOutVisTileData(OutData, CompactedTile, VT_Coord, PackedCoord);

			FDepthRange DepthRange;
			DepthRange.MinZ = group_MinZ;
			DepthRange.MaxZ = group_MaxZ;
			StoreOutVisTileData(OutData, CompactedTile, VT_MinMaxDepth, PackDepthRange(DepthRange));
		}
	}
	GroupMemoryBarrierWithGroupSync();


	#if PERMUTATION_DEBUG
	if (bDebugEnabled)
	{
		float4 DebugColor = ColorRed;
		if (GroupThreadID == 0)
		{
			AddQuadSS(Ctx, TileCoord * BIN_TILE_SIZE, TileCoord * BIN_TILE_SIZE + BIN_TILE_SIZE, DebugColor);
			Print(Ctx, TEXT("TileCoord      :"), FontWhite); Print(Ctx, TileCoord, FontWhite); Newline(Ctx);
			Print(Ctx, TEXT("TotalPrimCount :"), FontWhite); Print(Ctx, TotalPrimCount, FontWhite); Newline(Ctx);
			Print(Ctx, TEXT("group_NumTiles :"), FontWhite); Print(Ctx, group_NumTiles, FontWhite); Newline(Ctx);
			Print(Ctx, TEXT("group_MinZ     :"), FontWhite); Print(Ctx, UnpackDepth(group_MinZ), FontWhite); Newline(Ctx);
			Print(Ctx, TEXT("group_MaxZ     :"), FontWhite); Print(Ctx, UnpackDepth(group_MaxZ), FontWhite); Newline(Ctx);
		}
	}
	#endif

	// 3. Copy PrimIDs to compacted memory
	{
		const uint NumInputTiles = min(group_NumTiles, 1024);

		{
			// 3.1 First process the LDS list of tiles
			{
				for (uint LDSIdx = 0; LDSIdx < NumInputTiles; ++LDSIdx)
				{
					const uint TileIdx = group_TilesToCompact[LDSIdx];

					const uint TilePrimOffset = TileIdx * 1024;
					const uint TilePrimCount = LoadVisTileData(InData, TileIdx, VT_PrimCount);

					if (GroupThreadID < TilePrimCount)
					{
						const float Depth = UnpackDepth(InDepths[TilePrimOffset + GroupThreadID]);
						const uint BinIndex = GetDepthBinIndex(Depth);
						InterlockedAdd(s_BinCount[BinIndex], 1);

						#if PERMUTATION_DEBUG
						if (0 && GroupThreadID == 0)
						{
							Print(Ctx, TEXT("Depth0 :"), FontWhite); Print(Ctx, Depth, FontWhite);
							Print(Ctx, TEXT(" - BinIndex : - "), FontWhite); Print(Ctx, BinIndex, FontWhite);
							Print(Ctx, TEXT(" - BinMinZ : - "), FontWhite); Print(Ctx, UnpackDepth(group_MinZ), FontWhite);
							Print(Ctx, TEXT(" - BinMaxZ : - "), FontWhite); Print(Ctx, UnpackDepth(group_MaxZ), FontWhite);
							Newline(Ctx);
						}
						#endif
					}
				}
			}

			GroupMemoryBarrierWithGroupSync();

			// 3.2 Prefix sum of bin count
			if (GroupThreadID == 0)
			{
				uint GlobalOffset = 0;
				for (uint It=0; It < COMPACTION_DEPTH_BUCKET;++It)
				{
					s_BinOffset[It] = GlobalOffset;
					GlobalOffset += s_BinCount[It];
				}

				#if PERMUTATION_DEBUG
				for (uint It2 = 0; It2 < COMPACTION_DEPTH_BUCKET; ++It2)
				{					
					if (s_BinCount[It2] > 0)
						Print(Ctx, TEXT("x"), FontWhite);
					else
						Print(Ctx, TEXT("."), FontWhite);

					if (It2 != 0 && (It2 % 32) == 0)
						Newline(Ctx);
				}
				Newline(Ctx);
				#endif
			}

			GroupMemoryBarrierWithGroupSync();

			// 3.3 Clear insertion counter
			if (GroupThreadID < COMPACTION_DEPTH_BUCKET)
			{
				s_BinCount[GroupThreadID] = 0;
			}

			GroupMemoryBarrierWithGroupSync();

			// 3.4 Insert primitive into bins
			{
				uint CurrentWriteOffset = group_PrimWriteOffset;
				for (uint LDSIdx = 0; LDSIdx < NumInputTiles; ++LDSIdx)
				{
					const uint TileIdx = group_TilesToCompact[LDSIdx];

					const uint TilePrimOffset = TileIdx * 1024;
					const uint TilePrimCount = LoadVisTileData(InData, TileIdx, VT_PrimCount);

					if (GroupThreadID < TilePrimCount)
					{
						const float Depth = UnpackDepth(InDepths[TilePrimOffset + GroupThreadID]);
						const uint BinIndex = GetDepthBinIndex(Depth);
						const uint GlobalOffset = s_BinOffset[BinIndex];
						uint LocalOffset = 0;
						InterlockedAdd(s_BinCount[BinIndex], 1, LocalOffset);
						const uint WriteIndex = group_PrimWriteOffset + GlobalOffset + LocalOffset;
						OutPrims[WriteIndex] = InPrims[TilePrimOffset + GroupThreadID];
					}

					CurrentWriteOffset += TilePrimCount;
				}
			}

			// 3.5 Check any remaning tiles (Unlikely?)
			//if (group_NumTiles > 1024)
			//{
			//	for (uint TileIdx = group_MaxLDSTileIdx + 1; TileIdx < NumTiles; ++TileIdx)
			//	{
			//		const uint TilePackedCoord = LoadVisTileData(InData, TileIdx, VT_Coord);
			//		if (PackedCoord == TilePackedCoord)
			//		{
			//			const uint TilePrimOffset = TileIdx * 1024;
			//			const uint TilePrimCount = LoadVisTileData(InData, TileIdx, VT_PrimCount);
			//
			//			if (GroupThreadID < TilePrimCount)
			//			{
			//				OutPrims[CurrentWriteOffset + GroupThreadID] = InPrims[TilePrimOffset + GroupThreadID];
			//			}
			//
			//			CurrentWriteOffset += TilePrimCount;
			//		}
			//	}
			//}
		}
	}
}

#endif // SHADER_RASTERCOMPUTE_COMPACTION

///////////////////////////////////////////////////////////////////////////

#define RASTER_DEPTH_BUCKET 64
#define RASTER_SEGMENT_COUNT 32
#define SEGMENT_COUNT_PER_GROUP 1024
#define INVALID_PRIM_ID 0xFFFFFFFF
#define INVALID_VELOCITY -1e8f

// For editing convenience
#if !SHADER_RASTERCOMPUTE_DEBUG && !SHADER_RASTERCOMPUTE_COMPACTION && !SHADER_RASTERCOMPUTE_BINNING && !SHADER_RASTERCOMPUTE_RASTER_MULTI_SAMPLE && !SHADER_RASTERCOMPUTE_DEPTH_GRID
#define SHADER_RASTERCOMPUTE_RASTER 1
#endif

#if SHADER_RASTERCOMPUTE_RASTER

Buffer<uint> 				VisTilePrims;
StructuredBuffer<uint>		VisTileWork;
StructuredBuffer<uint>		VisTileWorkCount;
Buffer<uint> 				VisTileArgs;
ByteAddressBuffer			VisTileData;
RWTexture2D<float4>			OutSceneColorTexture;
RWTexture2D<float4>			OutSceneVelocityTexture;
RWStructuredBuffer<uint>	RWWorkCounter;

#if PERMUTATION_DEBUG
RWTexture2D<uint>			OutHairCountTexture_ForDebug;
RWTexture2D<uint>			OutHairPixelCountPerTile_ForDebug;
#endif

int2						SampleLightingViewportResolution;
Texture2D<float4>			SampleLightingTexture;
Buffer<float4>				SampleVelocityBuffer;

Texture2D<float>			SceneDepthTexture;

// Depth|ID
groupshared uint2  s_Segments[1024];
groupshared uint2  s_Segments_Sorted[1024];
groupshared uint   s_Segments_Min;
groupshared uint   s_Segments_Max;
groupshared uint   s_Segments_ValidCount;
groupshared uint   s_SegmentsCount[RASTER_DEPTH_BUCKET];
groupshared uint   s_SegmentsAlloc[RASTER_DEPTH_BUCKET];

//groupshared uint   s_Mask[RASTER_THREAD_COUNT];
groupshared uint   s_Mask[8][8];
groupshared uint2  s_OpaqueMask;
groupshared uint   s_Data[RASTER_SEGMENT_COUNT][8];
groupshared uint   s_Color[RASTER_SEGMENT_COUNT][8];
groupshared uint2  s_Velocity[RASTER_SEGMENT_COUNT][8];
groupshared uint   s_bDataOrder;
groupshared uint   s_WorkID;

groupshared uint   s_BinTileOffset;
groupshared uint   s_BinTileCount;

#if PERMUTATION_DEBUG
groupshared float s_Coverage[RASTER_THREAD_COUNT];
#endif

void ClearMask(uint2 In)
{
	s_Mask[In.x][In.y] = 0;
	//s_Mask[In.x + In.y * RASTER_TILE_SIZE] = 0;
}

uint ReadMask(uint2 In)
{
	return s_Mask[In.x][In.y];
	//return s_Mask[In.x + In.y * RASTER_TILE_SIZE];
}

void WriteMask(uint2 In, uint InValue)
{
	InterlockedOr(s_Mask[In.x][In.y], InValue);
	//InterlockedOr(s_Mask[In.x + In.y * RASTER_TILE_SIZE], InValue);
}

#if PERMUTATION_DEBUG
void PrintCoverage(inout FShaderPrintContext Ctx)
{
	Print(Ctx, TEXT("Coverage"), FontWhite);
	Newline(Ctx);
	for (uint y = 0; y < RASTER_TILE_SIZE; ++y)
	{
		for (uint x = 0; x < RASTER_TILE_SIZE; ++x)
		{
			const uint ValidSegments = ReadMask(uint2(x,y)); //s_Mask[x][y];
			if (ValidSegments != 0)
			{
				Print(Ctx, TEXT("x "), FontGreen);
			}
			else
			{
				Print(Ctx, TEXT(". "), FontRed);
			}
		}
		Newline(Ctx);
	}
}

void PrintPixelMask(inout FShaderPrintContext Ctx, uint InMask)
{
	for (uint s = 0; s < RASTER_SEGMENT_COUNT; ++s)
	{
		const bool bValid = ((1u<<s) & InMask) != 0u;
		if (bValid)
		{
			Print(Ctx, TEXT("x"), FontGreen);
		}
		else
		{
			Print(Ctx, TEXT("."), FontRed);
		}
	}
}

void PrintOpaqueMask(inout FShaderPrintContext Ctx, uint2 InOpaqueMask)
{
	for (uint y = 0; y < RASTER_TILE_SIZE; ++y)
	for (uint x = 0; x < RASTER_TILE_SIZE; ++x)
	{
		const uint s = x + y * RASTER_TILE_SIZE;
		bool bValid = false;
		if (s < 32) { bValid = ((1u << s) & InOpaqueMask.x) != 0u; }
		else { bValid = ((1u << (s - 32)) & InOpaqueMask.y) != 0u; }

		if (bValid) { Print(Ctx, TEXT("x "), FontGreen); }
		else { Print(Ctx, TEXT(". "), FontRed); }

		if (x == RASTER_TILE_SIZE-1) Newline(Ctx);
	}
}
#endif

struct FSampleData
{
	uint2 Coord;	// 2x3bits - Pixel coord within the tile 8x8
	float Depth;	// 16 bits
	float Coverage;	// 10 bits
};

uint PackSampleData(FSampleData In)
{
	return f32tof16(In.Depth) | (uint(saturate(In.Coverage) * 0x3FF) << 16u) | ((In.Coord.x & 0x7) << 26u) | ((In.Coord.y & 0x7) << 29u);
}

FSampleData UnpackSampleData(uint In)
{
	FSampleData Out;
	Out.Depth = f16tof32(In & 0xFFFF);
	Out.Coverage = ((In >> 16) & 0x3FF) * (1.f / 1023.f);
	Out.Coord = uint2((In >> 26) & 0x7, (In >> 29) & 0x7);
	return Out;
}

uint PackColorData(float3 In)
{
	return PackR11G11B10F(In);
}

float3 UnpackColorData(uint In)
{
	return UnpackR11G11B10F(In);
}

uint2 PackVelocityData(float4 In)
{
	return uint2(PackFloat2ToUInt(In.xy), PackFloat2ToUInt(In.zw));
}

float4 UnpackVelocityData(uint2 In)
{
	return float4(UnpackFloat2FromUInt(In.x), UnpackFloat2FromUInt(In.y));
}

float3 LoadSampleColor(uint InPrimId, uint2 InSampleResolution)
{
	const uint2 SampleCoord = GetHairSampleCoord(InPrimId, InSampleResolution);
	return SampleLightingTexture.Load(uint3(SampleCoord, 0)).xyz;
}

float4 LoadSampleVelocity(uint InPrimId)
{
	// This return the encoded velocity
	// For decoding the actual velocity, use DecodeVelocityFromTexture(...)
	return SampleVelocityBuffer[InPrimId];
}

uint GetDepthBinIndex(float InDepth, float InvDepthExtent)
{
	// Inverse-Z
	const float MinDepth = UnpackDepth(s_Segments_Min);
	const float MaxDepth = UnpackDepth(s_Segments_Max);
	const uint DepthIt = clamp(saturate((InDepth - MinDepth) * InvDepthExtent) * RASTER_DEPTH_BUCKET, 0, RASTER_DEPTH_BUCKET - 1);
	return (RASTER_DEPTH_BUCKET-1) - DepthIt;
}

#if PERMUTATION_DEBUG
void ShiftX(inout FShaderPrintContext Out, uint InPixelCountX)
{	
	const float fShift = float(InPixelCountX) / float(ShaderPrintData.Resolution.x);
	Out.StartPos.x += fShift;
	Out.Pos.x += fShift;
}

void ShiftY(inout FShaderPrintContext Out, uint InPixelCountY)
{
	const float fShift = float(InPixelCountY) / float(ShaderPrintData.Resolution.y);
	Out.StartPos.y += fShift;
	Out.Pos.y += fShift;
}
#endif

[numthreads(RASTER_TILE_SIZE, RASTER_TILE_SIZE, 1)]
void RasterCS(
	uint  GroupThread1D    : SV_GroupIndex,		/* 64  */
	uint2 GroupThread2D    : SV_GroupThreadID,  /* 8x8 */
	uint  GroupID          : SV_GroupID)		/* Rasterizer ID */
{
	ResolvedView = ResolveView();
	const uint FetchWorkCount = VisTileWorkCount[0];
	const uint BinTileNum = VisTileArgs[0];

	s_BinTileOffset = 0;
	s_BinTileCount  = 0;

	// These are global Color/Coverage for the final pixel handled by this thread
	float3 Thread_Color = 0;
	float  Thread_Coverage = 0;
	uint   Thread_Complete = 0;
	float4 Thread_Velocity = INVALID_VELOCITY;
	uint   Thread_LoopCountToFullCoverage = 0;
	uint2  Thread_PixelCoord = 0;

	const uint2 SampleLightingEffectiveResolution = GetHairSampleResolution(ControlPointCount[0]);

	LOOP
	for (uint WorkIndex = 0; WorkIndex < MAX_WORK_COUNT; WorkIndex++)
	{
		#if PERMUTATION_DEBUG
		FShaderPrintContext GlobalCtx = InitShaderPrintContext(false, uint2(0, 0));
		#endif

		// 0.1 Fetch work item
		if (GroupThread1D == 0)
		{
			InterlockedAdd(RWWorkCounter[0], 1, s_WorkID);
		}

		if (GroupThread1D == 0)
		{
			const uint FetchWorkIndex = s_WorkID / 16; // 1 x Bin32x32 -> 16 x Raster8x8

			uint2 Work = 0;
			if (FetchWorkIndex < FetchWorkCount)
			{
				Work = UnpackWork(VisTileWork[FetchWorkIndex]);
			}

			s_BinTileOffset = Work.x;
			s_BinTileCount  = Work.y;
			s_OpaqueMask    = 0;
		}

		GroupMemoryBarrierWithGroupSync();

		// 0.3 If we start a new screen tile, clear out final output
		{
			Thread_Color = 0;
			Thread_Coverage = 0;
			Thread_Complete = 0;
			Thread_Velocity = INVALID_VELOCITY;
			Thread_LoopCountToFullCoverage = 0;
			Thread_PixelCoord = 0;
		}

		// Early out if we are done
		{
			const uint FetchWorkIndex = s_WorkID / 16; // 1 x Bin32x32 -> 16 x Raster8x8
			if (FetchWorkIndex >= FetchWorkCount)
			{
				return;
			}
		}

		// Iterate over all bins for the current raster/screen tile
		const uint FrontToBackCount = min(s_BinTileCount, 64);
		uint ExitFrontToBackIndex = FrontToBackCount;
		for (uint FrontToBackIndex = 0; FrontToBackIndex < FrontToBackCount; ++FrontToBackIndex)
		{
			// 0.2 Reset all LDS variables
			{
				if (GroupThread1D == 0)
				{
					s_Segments_Min = PackDepth(1e8);
					s_Segments_Max = 0;
					s_Segments_ValidCount = 0;
					s_bDataOrder = 0;
				}
				if (GroupThread1D < RASTER_DEPTH_BUCKET)
				{
					s_SegmentsCount[GroupThread1D] = 0;
					s_SegmentsAlloc[GroupThread1D] = 0;
				}
				ClearMask(GroupThread2D);
				//s_Mask[Thread_Coord.x][Thread_Coord.y] = 0;

				if (GroupThread1D < 32)
				{
					s_Data[GroupThread1D][0] = 0;
					s_Data[GroupThread1D][1] = 0;
					s_Data[GroupThread1D][2] = 0;
					s_Data[GroupThread1D][3] = 0;
				}
				else
				{
					s_Data[GroupThread1D-32][4] = 0;
					s_Data[GroupThread1D-32][5] = 0;
					s_Data[GroupThread1D-32][6] = 0;
					s_Data[GroupThread1D-32][7] = 0;
				}

				#if PERMUTATION_DEBUG
				s_Coverage[GroupThread1D] = 0;
				#endif
			}
			GroupMemoryBarrierWithGroupSync();

			// 0.4 Early out when running out of valid tiles
			const uint BinTileIndex = s_BinTileOffset + FrontToBackIndex;
			const bool bTileValid = (BinTileIndex < BinTileNum);
			if (!bTileValid)
			{
				return;
			}

			const uint PrimOffset	= BinTileIndex * SEGMENT_COUNT_PER_GROUP;
			const uint PrimCount	= LoadVisTileData(VisTileData, BinTileIndex, VT_PrimCount);
			const uint2 BinTileCoord= UnpackVisTileCoord(LoadVisTileData(VisTileData, BinTileIndex, VT_Coord));
			const uint2 BinTileMin	= BinTileCoord * BIN_TILE_SIZE;
			const uint2 BinTileMax	= BinTileMin + BIN_TILE_SIZE;

			const uint QuadrantIndex	= s_WorkID % 16; // 1 x Bin32x32 -> 16 x Raster8x8
			const uint2 QuadrantCoord	= LinearTo2D_Common(QuadrantIndex, 4, 1.f / 4.f);
			const uint2 RasterTileCoord = BinTileCoord * 4 + QuadrantCoord;
			const uint2 RasterTileMin	= RasterTileCoord * RASTER_TILE_SIZE;
			const uint2 RasterTileMax	= RasterTileMin   + RASTER_TILE_SIZE;

			const uint LoopCount64  = DivideAndRoundUp(PrimCount, RASTER_THREAD_COUNT);

			Thread_PixelCoord = RasterTileMin + GroupThread2D;

			if (all(s_OpaqueMask == 0xFFFFFFFF))
			{
				ExitFrontToBackIndex = FrontToBackIndex;
				break;
			}

			// For debug only
			#if PERMUTATION_DEBUG
			const bool bDebugEnableAll	= FrontToBackIndex == 0 && all(RasterTileCoord == uint2(ShaderPrintData.CursorCoord / float(RASTER_TILE_SIZE)));
			const bool bDebugEnable		= FrontToBackIndex == 0 && all(RasterTileCoord == uint2(ShaderPrintData.CursorCoord / float(RASTER_TILE_SIZE))) && GroupThread1D == 0;
			FShaderPrintContext Ctx		= InitShaderPrintContext(bDebugEnable, uint2(350, 50));
			const FFontColor FontLegend = FontWhite;
			const FFontColor FontValue  = FontOrange;

			//const bool bDebugEnable2 = all(RasterTileCoord == uint2(ShaderPrintData.CursorCoord / float(RASTER_TILE_SIZE))) && GroupThread1D == 1;
			FShaderPrintContext CtxAll = InitShaderPrintContext(bDebugEnableAll, uint2(750, 50));
			#endif

			#if PERMUTATION_DEBUG
			if (bDebugEnable)
			{
				Print(Ctx, TEXT("Work - Index :"), FontLegend); Print(Ctx, FrontToBackIndex, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Work - Offset:"), FontLegend); Print(Ctx, s_BinTileOffset, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Work - Count :"), FontLegend); Print(Ctx, s_BinTileCount, FontValue); Newline(Ctx);

				//AddFilledQuadSS(BinTileMin, BinTileMax, Transparent(ColorLightGreen));
				//AddFilledQuadSS(RasterTileMin, RasterTileMax, Transparent(ColorLightGreen));
				AddQuadSS(BinTileMin, BinTileMax, Transparent(ColorLightGreen));
				AddQuadSS(RasterTileMin, RasterTileMax, Transparent(ColorYellow));
			}
			#endif

			// 1. Load all the segments and compute min/max bound (move this during the compaction)
			{
				LOOP
				for (uint LoopIndex = 0; LoopIndex < LoopCount64; ++LoopIndex)
				{
					const uint Prim = GroupThread1D + LoopIndex * RASTER_THREAD_COUNT;

					// Need to reset s_Segments and s_Segments_Sorted, as they are used for tracking valid segments
					s_Segments[Prim] = uint2(0, INVALID_PRIM_ID);
					s_Segments_Sorted[Prim] = uint2(0, INVALID_PRIM_ID);

					if (Prim < PrimCount)
					{
						uint PrimID = VisTilePrims[PrimOffset + Prim];

						uint TypeDummy = 0;
						float4 SP0 = 0;
						float4 SP1 = 0;
						float Rad0 = 0;
						float Rad1 = 0;
						CalcHomogenousPosAndRad(PrimID,   SP0, Rad0, TypeDummy);
						CalcHomogenousPosAndRad(PrimID+1, SP1, Rad1, TypeDummy);
				
						float Alpha0 = 0;
						float Alpha1 = 1;
						// Clipping
						{
							SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
							SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);

							// Clip against tile
							bool2 bClipped = false;
							bool bIsValidSegment = false;
							float2 T = 0;
							bIsValidSegment = ClipRaySegment(RasterTileMin - 0.5f, RasterTileMax + 0.5f, SP0, SP1, Rad0, Rad1, Alpha0, Alpha1, bClipped, T);

							PrimID = bIsValidSegment ? PrimID : INVALID_PRIM_ID;
						}

						const uint uDepth = PackDepth(max(SP0.z, SP1.z)); // Inverse Z
						s_Segments[Prim] = uint2(uDepth, PrimID);
						if (PrimID != INVALID_PRIM_ID)
						{
							InterlockedMin(s_Segments_Min, uDepth);
							InterlockedMax(s_Segments_Max, uDepth);
						}

						#if 0 && PERMUTATION_DEBUG
						if (bDebugEnableAll && PrimID != INVALID_PRIM_ID)
						{
							const float3 P0 = UnpackHairControlPoint(PrimID).Position;
							const float3 P1 = UnpackHairControlPoint(PrimID+1).Position;

							const float4 Color0 = float4(LoadSampleColor(PrimID, SampleLightingEffectiveResolution), 1);
							const float4 Color1 = float4(LoadSampleColor(PrimID+1, SampleLightingEffectiveResolution), 1);

							const float4 LineColor = float4(ColorMapTurbo(GroupThread1D / float(RASTER_THREAD_COUNT)), 1);
							AddLineWS(P0, P1, LineColor);
							//AddLineWS(P0, P1, Color0, Color1);
						}
						#endif
					}
				}
			}
			GroupMemoryBarrierWithGroupSync();
			const float InvDepthExtent = 1.f / max(0.0001f, UnpackDepth(s_Segments_Max)- UnpackDepth(s_Segments_Min));

			// 2. Compute the count of depth bucket
			{
				LOOP
				for (uint LoopIndex = 0; LoopIndex < LoopCount64; LoopIndex++)
				{
					const uint Prim = GroupThread1D + LoopIndex * RASTER_THREAD_COUNT;
					if (Prim < PrimCount)
					{
						const bool bIsValid = s_Segments[Prim].y != INVALID_PRIM_ID;
						if (bIsValid)
						{
							const float Depth	= UnpackDepth(s_Segments[Prim].x);
							const uint DepthIt  = GetDepthBinIndex(Depth, InvDepthExtent);
							InterlockedAdd(s_SegmentsCount[DepthIt], 1);
						}
					}
				}
			}
			GroupMemoryBarrierWithGroupSync();

			// Replace this with parallel version
			s_Segments_ValidCount = 0;
			if (GroupThread1D == 0)
			{
				uint Acc = 0;
				for (uint It = 0; It < RASTER_DEPTH_BUCKET; It++)
				{
					const uint Next = s_SegmentsCount[It];
					s_SegmentsCount[It] = Acc;
					Acc += Next;

					#if 1 && PERMUTATION_DEBUG
					if (bDebugEnable)
					{
						if (Next > 0)	{ Print(Ctx, TEXT("x"), FontValue); }
						else			{ Print(Ctx, TEXT("."), FontValue); }
						if (It == RASTER_DEPTH_BUCKET-1) { Newline(Ctx); }
					}
					#endif
				}
				s_Segments_ValidCount = Acc;
			}
			GroupMemoryBarrierWithGroupSync();

			// 3. Insert the segment into the right bucket
			{
				LOOP
				for (uint LoopIndex = 0; LoopIndex < LoopCount64; LoopIndex++)
				{
					const uint Prim = GroupThread1D + LoopIndex * RASTER_THREAD_COUNT;
					if (Prim < PrimCount)
					{
						const uint2 Segment = s_Segments[Prim];
						if (Segment.y != INVALID_PRIM_ID)
						{
							const float Depth = UnpackDepth(Segment.x);
							const uint DepthIt = GetDepthBinIndex(Depth, InvDepthExtent);

							uint AllocOffset = 0;
							InterlockedAdd(s_SegmentsAlloc[DepthIt], 1, AllocOffset);
							const uint NewIndex = AllocOffset + s_SegmentsCount[DepthIt];
							s_Segments_Sorted[NewIndex] = Segment;
						}
					}
				}
			}
			GroupMemoryBarrierWithGroupSync();

			const uint LoopCount32 = DivideAndRoundUp(s_Segments_ValidCount, RASTER_SEGMENT_COUNT);
			const uint LoopCount32_All = DivideAndRoundUp(PrimCount, RASTER_SEGMENT_COUNT);

			// DEBUG
			#if PERMUTATION_DEBUG
			if (bDebugEnable)
			{
				const uint2 OutCoord = GroupThread2D + RasterTileMin;

				//Print(Ctx, TEXT("Out Coord  :"), FontLegend); Print(Ctx, OutCoord, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Min Raster :"), FontLegend); Print(Ctx, RasterTileMin, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Thread     :"), FontLegend); Print(Ctx, Thread_Coord, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Max Raster :"), FontLegend); Print(Ctx, RasterTileMax, FontValue); Newline(Ctx);
				//Newline(Ctx);

				//Print(Ctx, TEXT("Cursor     :"), FontLegend); Print(Ctx, uint2(ShaderPrintData.CursorCoord), FontValue); Newline(Ctx);
				//Newline(Ctx);

				Print(Ctx, TEXT("Work ID      :"), FontLegend); Print(Ctx, s_WorkID, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Prim Count   :"), FontLegend); Print(Ctx, s_Segments_ValidCount, FontValue);	Print(Ctx, TEXT(" / "), FontLegend); Print(Ctx, PrimCount, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Loop 32 Count:"), FontLegend); Print(Ctx, LoopCount32, FontValue);				Print(Ctx, TEXT(" / "), FontLegend); Print(Ctx, LoopCount32_All, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Loop 64 Count:"), FontLegend); Print(Ctx, LoopCount64, FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Min Depth    :"), FontLegend); Print(Ctx, UnpackDepth(s_Segments_Min), FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Max Depth    :"), FontLegend); Print(Ctx, UnpackDepth(s_Segments_Max), FontValue); Newline(Ctx);
				Print(Ctx, TEXT("Rad.at Depth1:"), FontLegend); Print(Ctx, RadiusAtDepth1, FontValue); Newline(Ctx);
				Newline(Ctx);

				Print(Ctx, TEXT("Work - Index :"), FontLegend); Print(Ctx, WorkIndex, FontValue, 2, 0);
				Print(Ctx, TEXT(" - Offset:"), FontLegend); Print(Ctx, s_BinTileOffset, FontValue, 5, 0);
				Print(Ctx, TEXT(" - Count :"), FontLegend); Print(Ctx, s_BinTileCount, FontValue, 3, 0);
				Newline(Ctx);

				//Print(Ctx, TEXT("Bin ID     :"), FontLegend); Print(Ctx, BinTileIndex, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Raster ID  :"), FontLegend); Print(Ctx, QuadrantIndex, FontValue); Newline(Ctx);
				//Newline(Ctx);

				//Print(Ctx, TEXT("Raster Tile:"), FontLegend); Print(Ctx, RasterTileCoord, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Bin Tile   :"), FontLegend); Print(Ctx, BinTileCoord, FontValue); Newline(Ctx);
				//Print(Ctx, TEXT("Out Coord  :"), FontLegend); Print(Ctx, OutCoord, FontValue); Newline(Ctx);
				//Newline(Ctx);

			}
			#endif

			if (LoopCount32 == 0)
			{
				continue;
			}

			// 4. Raster segments
			{
				// 4.1 Loop over all segment within the tile, and rastize 32 of them at each loop
				LOOP
				for (uint LoopIndex = 0; LoopIndex < LoopCount32; LoopIndex++)
				{
					// 4.0 Reset
					if (GroupThread1D < RASTER_SEGMENT_COUNT)
					{
						for (int J = 0; J < RASTER_TILE_SIZE; ++J)
						{
							s_Data[GroupThread1D][J] = 0;
							s_Color[GroupThread1D][J] = 0;
							s_Velocity[GroupThread1D][J] = 0;
						}
					}
					s_Mask[GroupThread2D.x][GroupThread2D.y] = 0;
					s_bDataOrder = 0;

					GroupMemoryBarrierWithGroupSync();

					// If raster tile is fully covered, exit
					if (all(s_OpaqueMask == 0xFFFFFFFF))
					{
						break;
					}

					// 4.1 Raster segment (1 thread = 1 segment)
					// Half of the thread are doing nothing, we could raster the two half of the segments (one per each thread)
					const uint Prim = GroupThread1D + LoopIndex * RASTER_SEGMENT_COUNT;
					if (GroupThread1D < RASTER_SEGMENT_COUNT && Prim < s_Segments_ValidCount)
					{
						const uint PrimID = s_Segments_Sorted[Prim].y;
						uint TypeDummy = 0;
						float4 SP0 = 0;
						float4 SP1 = 0;
						float Rad0 = 0;
						float Rad1 = 0;
						CalcHomogenousPosAndRad(PrimID,     SP0, Rad0, TypeDummy);
						CalcHomogenousPosAndRad(PrimID + 1, SP1, Rad1, TypeDummy);

						float Alpha0=0;
						float Alpha1=1;

						bool2 bClipped = false;
						bool bIsSegmentValid = false;
						float2 T = 0;
						// 4.1.1 Clipping
						{
							SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
							SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);
				
							// Clip against tile
							bIsSegmentValid = ClipRaySegment(RasterTileMin - 0.5f, RasterTileMax + 0.5f, SP0, SP1, Rad0, Rad1, Alpha0, Alpha1, bClipped, T);
						}

						// DEBUG (Write coord)
						#if 0 && PERMUTATION_DEBUG
						if (bDebugEnableAll)
						{
							ShiftY(CtxAll, 500 + GroupThread1D * 150);
						
							Print(CtxAll, TEXT("Thrd 1D:"), FontYellow);
							Print(CtxAll, GroupThread1D, FontWhite);
							Newline(CtxAll);

							Print(CtxAll, TEXT("ID     :"), FontYellow);
							Print(CtxAll, PrimID, FontWhite);
							Newline(CtxAll);

							Print(CtxAll, TEXT("Valid  :"), FontYellow);
							PrintBool(CtxAll, bIsSegmentValid);
							Newline(CtxAll);

							Print(CtxAll, TEXT("Clip X :"), FontYellow);
							PrintBool(CtxAll, bClipped.x);
							Newline(CtxAll);

							Print(CtxAll, TEXT("Clip Y :"), FontYellow);
							PrintBool(CtxAll, bClipped.y);
							Newline(CtxAll);

							Print(CtxAll, TEXT("X0      :"), FontYellow);
							Print(CtxAll, SP0, FontWhite);
							Newline(CtxAll);

							Print(CtxAll, TEXT("X1      :"), FontYellow);
							Print(CtxAll, SP1, FontWhite);
							Newline(CtxAll);

							//const float3 P0 = UnpackHairControlPoint(PrimID).Position;
							//const float3 P1 = UnpackHairControlPoint(PrimID + 1).Position;

							//const float4 Color0 = float4(LoadSampleColor(PrimID), 1);
							//const float4 Color1 = float4(LoadSampleColor(PrimID + 1), 1);

							//const float4 LineColor = float4(ColorMapTurbo(GroupThread1D / float(RASTER_THREAD_COUNT)), 1);
							//AddLineWS(P0, P1, LineColor);
							if (GroupThread1D == 0)
								AddLineSS(SP0, SP1, ColorPurple);
						}
						#endif

						// 4.1.2 Rasterize segment (1 thread = 1 segments)
						if (bIsSegmentValid)
						{
							const float SegmentLenSqRcp = 1.0f / dot(SP1.xy - SP0.xy, SP1.xy - SP0.xy);
							const bool bIsSteep = abs(SP1.x - SP0.x) < abs(SP1.y - SP0.y);
							const float X0 = bIsSteep ? min(SP0.y, SP1.y) : min(SP0.x, SP1.x);
							const float X1 = bIsSteep ? max(SP0.y, SP1.y) : max(SP0.x, SP1.x);
							const float RcpNumSteps = 1.0f / (X1 - X0);
							const int NumSteps = (int)(ceil(X1) - floor(X0));

							if (bIsSteep)
							{
								InterlockedOr(s_bDataOrder, 1u << GroupThread1D);
							}

							// DEBUG (Write coord)
							#if 0 && PERMUTATION_DEBUG
							//if (GroupThread1D == 0 && bDebugEnable)
							if (bDebugEnableAll)
							{							
								ShiftY(CtxAll, GroupThread1D * 150);
								//ShiftY(CtxAll, GroupThread1D * )

								Print(CtxAll, TEXT("NumSteps:"), FontYellow);
								Print(CtxAll, NumSteps, FontWhite);
								Newline(CtxAll);

								Print(CtxAll, TEXT("X0      :"), FontYellow);
								Print(CtxAll, X0, FontWhite);
								Newline(CtxAll);

								Print(CtxAll, TEXT("X1      :"), FontYellow);
								Print(CtxAll, X1, FontWhite);
								Newline(CtxAll);

								Print(CtxAll, TEXT("T       :"), FontYellow);
								Print(CtxAll, T, FontWhite);
								Newline(CtxAll);

								Print(CtxAll, TEXT("bValid  :"), FontYellow);
								PrintBool(CtxAll, bIsSegmentValid);
								Newline(CtxAll);

								Print(CtxAll, TEXT("bClip X :"), FontYellow);
								PrintBool(CtxAll, bClipped.x);
								Newline(CtxAll);

								Print(CtxAll, TEXT("bClip Y :"), FontYellow);
								PrintBool(CtxAll, bClipped.y);
								Newline(CtxAll);
							
							
								//const float3 P0 = UnpackHairControlPoint(PrimID).Position;
								//const float3 P1 = UnpackHairControlPoint(PrimID + 1).Position;

								//const float4 Color0 = float4(LoadSampleColor(PrimID, SampleLightingEffectiveResolution), 1);
								//const float4 Color1 = float4(LoadSampleColor(PrimID + 1, SampleLightingEffectiveResolution), 1);

								//const float4 LineColor = float4(ColorMapTurbo(GroupThread1D / float(RASTER_THREAD_COUNT)), 1);
								//AddLineWS(P0, P1, LineColor);
								AddLineSS(SP0, SP1, ColorRed);
							}
							#endif

							// DEBUG (Legend)
							#if 0 && PERMUTATION_DEBUG
							if (bDebugEnable)
							{
								//Print(Ctx, s_Mask[IntraTileCoord.x][IntraTileCoord.y], FontWhite);
								//PrintPixelMask(Ctx, ReadMask(IntraTileCoord)/*s_Mask[IntraTileCoord.x][IntraTileCoord.y]*/);
								//Print(Ctx, TEXT("Color0: "), FontWhite); Print(Ctx, Color0); Newline(Ctx);
								//Print(Ctx, TEXT("Color1: "), FontWhite); Print(Ctx, Color1); Newline(Ctx);
								//Print(Ctx, TEXT("Color :  "), FontWhite); Print(Ctx, Color);
								const float3 Color0 = LoadSampleColor(PrimID, SampleLightingEffectiveResolution);
								const float3 Color1 = LoadSampleColor(PrimID + 1, SampleLightingEffectiveResolution);
								Print(Ctx, TEXT("Color0 :  "), FontWhite); Print(Ctx, Color0); Newline(Ctx);
								Print(Ctx, TEXT("Color1 :  "), FontWhite); Print(Ctx, Color1); Newline(Ctx);
								Print(Ctx, TEXT("  Alpha       A0          A1          AColor     Color    Coord  Coverage"), FontWhite);
								Newline(Ctx);
							}
							#endif
							LOOP
							for (int J = 0; J < NumSteps; ++J)
							//for (int J = 0; J < RASTER_TILE_SIZE; ++J)
							{
								const float AlphaSP = saturate(J * RcpNumSteps);
								const float4 SP = lerp(SP0, SP1, AlphaSP);

								int2 Coords = SP.xy;
								const int2 IntraTileCoord = Coords - RasterTileMin;

								// TO it needs to store data in sweeping order 
								FSampleData Sample = (FSampleData)0;
								if (all(IntraTileCoord >= 0) && all(IntraTileCoord < RASTER_TILE_SIZE))
								{
									const float Alpha = ComputeLerpAlpha(Coords, SP0.xy, SP1.xy, SegmentLenSqRcp);
									const float Depth = lerp(SP0.z, SP1.z, Alpha);
									const float OpaqueDepth = SceneDepthTexture.Load(uint3(Coords, 0));
									if (Depth > OpaqueDepth)
									{
										const float AlphaColor = lerp(Alpha0, Alpha1, Alpha);
										const float3 Color0 = LoadSampleColor(PrimID, SampleLightingEffectiveResolution);
										const float3 Color1 = LoadSampleColor(PrimID+1, SampleLightingEffectiveResolution);
										const float3 Color = lerp(Color0, Color1, AlphaColor);

										const float4 Velocity0 = LoadSampleVelocity(PrimID);
										const float4 Velocity1 = LoadSampleVelocity(PrimID+1);
										const float4 Velocity = lerp(Velocity0, Velocity1, AlphaColor);


										// Fill in sample data
										Sample.Depth = Depth;
										Sample.Coord = IntraTileCoord;

										// Compute coverage
										// Minimal radius to snap the strand to a sample/pixel center (to avoid aliasing)
										const float SceneDistance = ConvertFromDeviceZ(Depth);
										const float MinHairRadius = ConvertGivenDepthRadiusForProjectionType(RadiusAtDepth1, SceneDistance);
										const float HairRadius = lerp(Rad0, Rad1, AlphaColor);
										Sample.Coverage = saturate(HairRadius / MinHairRadius);

										// Write data
										WriteMask(IntraTileCoord, 1u << GroupThread1D);

										const uint JCoord = bIsSteep ? IntraTileCoord.y : IntraTileCoord.x;
										s_Color[GroupThread1D][JCoord] = PackColorData(Color);
										s_Data[GroupThread1D][JCoord]  = PackSampleData(Sample);
										s_Velocity[GroupThread1D][JCoord] = PackVelocityData(Velocity);


										// DEBUG (Write coord)
										#if 0 && PERMUTATION_DEBUG
										if (bDebugEnable)
										{
											Print(Ctx, J, FontYellow, 2, 0);
											Print(Ctx, Alpha, FontWhite);
											Print(Ctx, AlphaColor, FontBlue);
											Print(Ctx, HairRadius, FontRed);
											Print(Ctx, MinHairRadius, FontGreen);
											Print(Ctx, Sample.Coverage, FontBlue);
											Newline(Ctx);
										}
										#endif
									}
								}
							}
						} // if (bIsSegmentValid)
					}
					GroupMemoryBarrierWithGroupSync();

					// DEBUG (Coverage mask)
					#if 0 && PERMUTATION_DEBUG
					if (bDebugEnable)
					{
						PrintCoverage(Ctx);
					}
					#endif

					// 4.2 Combine all samples within the same pixel (1 thread = 1 pixel)
					{
						const uint ValidSegments = ReadMask(GroupThread2D);//s_Mask[Thread_Coord.x][Thread_Coord.y];

						// Change this loop into to bit logic?
						{
							for (uint SegmentIt=0; SegmentIt < RASTER_SEGMENT_COUNT; ++SegmentIt)
							{
								const uint SegmentBit = 1u << SegmentIt;
								const bool bIsValid = (ValidSegments & SegmentBit) != 0;
								const bool bIsSteep = (s_bDataOrder & SegmentBit) != 0;
								if (bIsValid)
								{
									const uint J = bIsSteep ? GroupThread2D.y : GroupThread2D.x;

									const FSampleData Sample = UnpackSampleData(s_Data[SegmentIt][J]);
									const float3 Color = UnpackColorData(s_Color[SegmentIt][J]);
									const float4 Velocity = UnpackVelocityData(s_Velocity[SegmentIt][J]);

									const float AccTransmittance = saturate(1.f-Thread_Coverage);
									Thread_Coverage += AccTransmittance * Sample.Coverage;
									Thread_Color    += AccTransmittance * Sample.Coverage * Color;

									// Use the closest valid segment as output velocity
									if (Thread_Velocity.x <= INVALID_VELOCITY)
									{
										Thread_Velocity = Velocity;
									}
								}
							}
						}

						const float CoverageThreshold = 0.95f;
						if (Thread_Complete == 0 && Thread_Coverage > CoverageThreshold)
						{
							// Mark pixel has fully covered
							if (GroupThread1D >= 32)
							{
								InterlockedOr(s_OpaqueMask.y, 1u << (GroupThread1D - 32));
							}
							else
							{
								InterlockedOr(s_OpaqueMask.x, 1u << GroupThread1D);
							}
							Thread_Complete = 1;
						}

						#if PERMUTATION_DEBUG
						if (Thread_LoopCountToFullCoverage == 0 && Thread_Coverage > CoverageThreshold)
						{
							Thread_LoopCountToFullCoverage = LoopIndex;
						}
						#endif
					}
					//GroupMemoryBarrierWithGroupSync();

				} // for (uint LoopIndex =...) ...

				#if PERMUTATION_DEBUG
				s_Coverage[GroupThread1D] = Thread_Coverage; // For sync s_Coverage
				GlobalCtx = Ctx;
				#endif

				GroupMemoryBarrierWithGroupSync();

			} // 4. Raster

		} // for (... FrontToBackIndex ...)

		// 5. Write final color
		const bool bWriteOut = Thread_Coverage > 0;
		if (bWriteOut)
		{
			const uint2 PixelCoord = Thread_PixelCoord;
			const float3 SourceColor = OutSceneColorTexture[PixelCoord].xyz;
			OutSceneColorTexture[PixelCoord] = float4(SourceColor * (1-Thread_Coverage) + Thread_Color, 1);
			if (Thread_Velocity.x > INVALID_VELOCITY)
			{
				OutSceneVelocityTexture[PixelCoord] = Thread_Velocity;
			}
			
			// For debug purpose only
			#if PERMUTATION_DEBUG
			{
				const uint2 RasterTileCoord = PixelCoord / RASTER_TILE_SIZE;
				InterlockedAdd(OutHairCountTexture_ForDebug[PixelCoord], 1);
				InterlockedAdd(OutHairPixelCountPerTile_ForDebug[RasterTileCoord], 1);

				const FFontColor FontLegend = FontWhite;
				const FFontColor FontValue = FontOrange;
				Print(GlobalCtx, TEXT("Exit :"), FontLegend);
				Print(GlobalCtx, ExitFrontToBackIndex, FontValue, 3, 0);
				Print(GlobalCtx, TEXT(" / "), FontLegend);
				Print(GlobalCtx, FrontToBackCount, FontValue, 3, 0);
				Newline(GlobalCtx);

				PrintOpaqueMask(GlobalCtx, s_OpaqueMask);
			}
			#endif
		}

	} // for ( ... Work item ... )
}

#endif //SHADER_RASTERCOMPUTE_RASTER

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_DEBUG

#include "../ShaderPrint.ush"

Buffer<uint>						VisTileArgs;
ByteAddressBuffer					VisTileData;

Buffer<uint>						CompactedVisTileArgs;
ByteAddressBuffer					CompactedVisTileData;

Texture2D<uint>						HairCountTexture_ForDebug;
Texture2D<uint>						HairPixelCountPerTile_ForDebug;
uint								InstanceCount;
uint								CPUAllocatedTileCount;
uint								CPUAllocatedCompactedTileCount;

uint GetTileTotalSegment(uint2 TileCoord, bool bPrintDetails, uint InTileSize)
{
	const float TileDisplayScale = 1.5f;
	const uint DisplayTileSize = InTileSize * TileDisplayScale;
	uint2 InlinedTileCoord = uint2(0, 0);

	const uint TileCount = VisTileArgs[0];

	uint TotalSegments = 0;
	for (uint TileIndex=0; TileIndex<TileCount; ++TileIndex)
	{
		const uint PackedTileCoord = LoadVisTileData(VisTileData, TileIndex, VT_Coord);
		const uint2 VisTileCoord = UnpackVisTileCoord(PackedTileCoord);
		if (all(VisTileCoord == TileCoord))
		{
			const uint TileSegments = LoadVisTileData(VisTileData, TileIndex, VT_PrimCount);
			TotalSegments += TileSegments;

			if (bPrintDetails)
			{
				AddFilledQuadSS(InlinedTileCoord * DisplayTileSize, (InlinedTileCoord + 1) * DisplayTileSize, TileSegments > 0 ? Transparent(ColorLightGreen) : Transparent(ColorLightRed));
				AddQuadSS(InlinedTileCoord * DisplayTileSize, (InlinedTileCoord + 1) * DisplayTileSize, ColorYellow);

				const uint2 TilePrintOffset = InTileSize >> 1;

				FShaderPrintContext Context = InitShaderPrintContext(true, InlinedTileCoord * DisplayTileSize + TilePrintOffset);
				Print(Context, TileSegments, FontWhite);
				++InlinedTileCoord.x;
			}
		}
	}
	return TotalSegments;
}

void PrintTile(uint2 TileCoord, uint TotalSegments, bool bPrintText, uint InTileSize)
{
	AddFilledQuadSS(TileCoord * InTileSize, (TileCoord + 1) * InTileSize, TotalSegments > 0 ? Transparent(ColorLightGreen) : Transparent(ColorLightRed));
	if (bPrintText)
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, TileCoord * InTileSize + uint2(0, InTileSize * 1.5f));
		Print(Context, TotalSegments, FontWhite);

		AddQuadSS(TileCoord * InTileSize, (TileCoord + 1) * InTileSize, ColorYellow);
	}
}

[numthreads(8, 8, 1)]
void MainCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const bool bIsCursorPixel = (all(ThreadId == 0) && all(ShaderPrintData.CursorCoord >= 0));

	// Info/Stats
	if (all(ThreadId == 0))
	{
		FFontColor FontValue = FontOrange;
		FFontColor FontTitle = FontYellow;
		FFontColor FontLegend = FontWhite;
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 110));
		Print(Context, TEXT("Raster compute         "), FontTitle); Newline(Context);
		Print(Context, TEXT("Instance Count       : "), FontLegend); Print(Context, InstanceCount, FontValue, 3, 0); Newline(Context);
		Print(Context, TEXT("Total segments Count : "), FontLegend); Print(Context, GetControlPointCount(), FontValue); Newline(Context);
		Print(Context, TEXT("Max.  segments Count : "), FontLegend); Print(Context, MaxControlPointCount, FontValue);  Newline(Context);
		Newline(Context);

		Print(Context, TEXT("Configuration          "), FontTitle); Newline(Context);
		Print(Context, TEXT("Output Resolution    : "), FontLegend); Print(Context, OutputResolution.x, FontValue, 4, 0); Print(Context, TEXT("x"), FontSilver); Print(Context, OutputResolution.y, FontValue, 4, 0); Newline(Context);
		Newline(Context);

		Print(Context, TEXT("Bin Tile Size        : "), FontLegend); Print(Context, uint(BIN_TILE_SIZE), FontValue, 2, 0); Newline(Context);
		Print(Context, TEXT("Bin Tile Res         : "), FontLegend); Print(Context, BinTileRes.x, FontValue, 3, 0); Print(Context, TEXT("x"), FontSilver); Print(Context, BinTileRes.y, FontValue, 3, 0); Newline(Context);
		Print(Context, TEXT("Num Binners          : "), FontLegend); Print(Context, NumBinners, FontValue); Newline(Context);
		Newline(Context);

		Print(Context, TEXT("Raster Tile Size     : "), FontLegend); Print(Context, uint(RASTER_TILE_SIZE), FontValue, 2, 0); Newline(Context);
		Print(Context, TEXT("Raster Tile Res      : "), FontLegend); Print(Context, RasterTileRes.x, FontValue, 3, 0); Print(Context, TEXT("x"), FontSilver); Print(Context, RasterTileRes.y, FontValue, 3, 0); Newline(Context);
		Print(Context, TEXT("Num Rasterizers      : "), FontLegend); Print(Context, NumRasterizers, FontValue); Newline(Context);
		Newline(Context);

		const FFontColor AllocColor			 = InitFontColor(ColorMapTurbo(VisTileArgs[0] / float(CPUAllocatedTileCount)));
		const FFontColor AllocCompactedColor = InitFontColor(ColorMapTurbo(CompactedVisTileArgs[0] / float(CPUAllocatedCompactedTileCount)));

		Print(Context, TEXT("Alloc. Tile          : "), FontLegend); Print(Context, VisTileArgs[0], AllocColor, 6, 0); Print(Context, TEXT(" / "), FontLegend); Print(Context, CPUAllocatedTileCount, FontValue,6,0); Newline(Context);
		Print(Context, TEXT("Alloc. Compacted Tile: "), FontLegend); Print(Context, CompactedVisTileArgs[0], AllocCompactedColor, 6, 0); Print(Context, TEXT(" / "), FontLegend); Print(Context, CPUAllocatedCompactedTileCount, FontValue, 6, 0); Newline(Context);
		Print(Context, TEXT("Rasterizer Max Work  : "), FontLegend); Print(Context, NumRasterizers * MAX_WORK_COUNT, FontValue); Newline(Context);
		Newline(Context);
		
		Newline(Context);

		if (bIsCursorPixel)
		{
		const uint2 PixelCoord = ShaderPrintData.CursorCoord;
		const uint2 RasterTileCoord = uint2(ShaderPrintData.CursorCoord) >> RASTER_TILE_SIZE_AS_SHIFT;

		const uint HairCount = HairCountTexture_ForDebug.Load(uint3(PixelCoord, 0));
		const uint RasterizedPixels = HairPixelCountPerTile_ForDebug.Load(uint3(RasterTileCoord, 0));

		Print(Context, TEXT("Hair Count           : "), FontLegend); Print(Context, HairCount, FontValue); Newline(Context);
		Print(Context, TEXT("Hair #Pixel in Tile  : "), FontLegend); Print(Context, RasterizedPixels, FontValue); Newline(Context);
		Newline(Context);
		}
	}
	#if 0
	// Cursor info
	if (bIsCursorPixel)
	{
		const uint2 PixelCoord = ShaderPrintData.CursorCoord;
		const uint2 BinTileCoord = uint2(ShaderPrintData.CursorCoord) >> BIN_TILE_SIZE_AS_SHIFT;
		if (all(BinTileCoord < BinTileRes))
		{
			const uint TotalSegments = GetTileTotalSegment(BinTileCoord, true, BIN_TILE_SIZE);
			PrintTile(BinTileCoord, TotalSegments, true, BIN_TILE_SIZE);
		}
	}

	// All tile
	{
		const uint2 BinTileCoord = ThreadId.xy;
		if (all(BinTileCoord < BinTileRes))
		{
			const uint TotalSegments = GetTileTotalSegment(BinTileCoord, false, BIN_TILE_SIZE);
			if (TotalSegments)
			{
				PrintTile(BinTileCoord, TotalSegments, false, BIN_TILE_SIZE);
			}
		}
	}
	#endif
}
#endif //SHADER_RASTERCOMPUTE_DEBUG

