// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Utils function for writing data into GBuffer textures
void WriteGBuffer(
	const float3 Tangent,
	const float Specular,
	const float3 BaseColor,
	const float Roughness,
	const float LightChannelMask,
	const float Backlit,
	const float Depth,
	const float2 Velocity,
	inout float4 OutGBufferA,
	inout float4 OutGBufferB,
	inout float4 OutGBufferC,
	inout float4 OutGBufferD,
	inout float4 OutGBufferE,
	inout float  OutDepth,
	inout float4 OutVelocity)
{
	const float SampleCoverage8bit = 1; // TODO?
	const float PerObjectData = 0;
	const float Irradiance_AO = 0;
	const float Metallic = 0;
	OutGBufferA = float4(EncodeNormal(Tangent), PerObjectData);
	OutGBufferB = float4(Metallic, Specular, Roughness, EncodeShadingModelIdAndSelectiveOutputMask(SHADINGMODELID_HAIR, LightChannelMask));
	OutGBufferC = float4(BaseColor, Irradiance_AO);
	OutGBufferD = float4(0, 0, Backlit, 0);
	OutGBufferE = float4(1, 1, 1, 1);
	OutVelocity = float4(Velocity, 0, 0);
	OutDepth = Depth;
}

void WriteGBuffer(
	const float3 Tangent,
	const float Specular,
	const float3 BaseColor,
	const float Roughness,
	const float LightChannelMask,
	const float Backlit,
	const float Depth,
	inout float4 OutGBufferA,
	inout float4 OutGBufferB,
	inout float4 OutGBufferC,
	inout float4 OutGBufferD,
	inout float4 OutGBufferE,
	inout float  OutDepth)
{
	float2 InDummyVelocity = 0;
	float4 OutDummyVelocity = 0;
	WriteGBuffer(
		Tangent,
		Specular,
		BaseColor,
		Roughness,
		LightChannelMask,
		Backlit,
		Depth,
		InDummyVelocity,
		OutGBufferA,
		OutGBufferB,
		OutGBufferC,
		OutGBufferD,
		OutGBufferE,
		OutDepth,
		OutDummyVelocity);
}