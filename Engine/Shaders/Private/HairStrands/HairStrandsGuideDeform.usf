// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsBindingCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

#if SHADER_DEFORM_GUIDE

float3 SimRestOffset;
uint VertexCount;
uint InstanceRegisteredIndex;

StructuredBuffer<float4> SimDeformedOffsetBuffer;
ByteAddressBuffer SimRestPosePositionBuffer;
RWByteAddressBuffer OutSimDeformedPositionBuffer;

#define DEFORMATION_BYPASS 0
#define DEFORMATION_OFFSET_GUIDE 1
#define DEFORMATION_OFFSET_GUIDE_DYNAMIC 2
#define DEFORMATION_OFFSET_GUIDE_GLOBAL 3
#define DEFORMATION_OFFSET_GUIDE_BONES 4

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_DYNAMIC

Buffer<float4>  SimRestPositionBuffer;
Buffer<float4>  SimDeformedPositionBuffer;

Buffer<uint>	  SimRootBarycentricBuffer;
ByteAddressBuffer SimPointToCurveBuffer;
Buffer<uint>	  SimRootToUniqueTriangleIndexBuffer;

// Return world position of a guide vertex deformed by a skinned triangle
float3 DeformGuideByTriangle(
	uint GuideIndex,
	FHairMeshBasis RestBasis,
	FHairMeshBasis DeformedBasis)
{
	const float3 WorldRestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex, SimRestOffset);
	const float3 LocalRestGuidePoint = ToTriangle(WorldRestGuidePoint, RestBasis);
	const float3 WorldDeformedGuidePoint = ToWorld(LocalRestGuidePoint, DeformedBasis);

	return WorldDeformedGuidePoint;
}

#endif

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_BONES

Buffer<float4>  BoneDeformedPositionBuffer;

#endif


#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_GLOBAL
uint SampleCount;
uint RBFLocalSpace;
StructuredBuffer<float4>  RestSamplePositionsBuffer;
StructuredBuffer<float4>  DeformedSamplePositionsBuffer;
StructuredBuffer<float4>  MeshSampleWeightsBuffer;
#endif

float3 DisplacePosition(uint VertexIndex, float3 Pos)
{
	const float3 OutSimHairPositionOffset = ReadSimPositionOffset(SimDeformedOffsetBuffer, InstanceRegisteredIndex);

// By pass
#if PERMUTATION_DEFORMATION == DEFORMATION_BYPASS
	return Pos;
#endif

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE
	return Pos + (SimRestOffset - OutSimHairPositionOffset);
#endif

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_DYNAMIC
	const uint RootIndex = ReadHairPointToCurveIndex(SimPointToCurveBuffer, VertexIndex);
	const float3 RootBarycentric = UnpackBarycentrics(SimRootBarycentricBuffer[RootIndex]);

	const uint TriangleIndex = SimRootToUniqueTriangleIndexBuffer[RootIndex];
	const FHairMeshBasis RestBasis = GetHairMeshBasis(TriangleIndex, SimRestPositionBuffer, RootBarycentric);
	const FHairMeshBasis DeformedBasis = GetHairMeshBasis(TriangleIndex, SimDeformedPositionBuffer, RootBarycentric);

	return DeformGuideByTriangle(VertexIndex, RestBasis, DeformedBasis) - OutSimHairPositionOffset;
#endif

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_GLOBAL
	const float3 RestControlPoint = Pos + SimRestOffset;
	const float3 ControlPoint = ApplyRBF(RestControlPoint, SampleCount, RestSamplePositionsBuffer, MeshSampleWeightsBuffer, DeformedSamplePositionsBuffer, RBFLocalSpace);
	return ControlPoint - OutSimHairPositionOffset;
#endif

#if PERMUTATION_DEFORMATION == DEFORMATION_OFFSET_GUIDE_BONES
	const float4 RestControlPoint = float4(Pos + SimRestOffset,1);
	const int BufferOffset = 3 * (VertexIndex+1);

	const float3x4 BoneMatrix = float3x4(BoneDeformedPositionBuffer[BufferOffset], BoneDeformedPositionBuffer[BufferOffset+1], BoneDeformedPositionBuffer[BufferOffset+2]);
	return mul(BoneMatrix,RestControlPoint) - OutSimHairPositionOffset;
#endif
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint VertexIndex = DispatchThreadId;
	
	if (VertexIndex < VertexCount)
	{
		const FPackedHairPosition PackedCP = ReadPackedHairPosition(SimRestPosePositionBuffer, VertexIndex);
		const float3 CPPosition			= UnpackHairControlPointPosition(PackedCP);
		const float3 DisplacedPosition	= DisplacePosition(VertexIndex, CPPosition);
		WritePackedHairControlPointPosition(OutSimDeformedPositionBuffer, VertexIndex, PackedCP, DisplacedPosition);
//		OutSimDeformedPositionBuffer[VertexIndex] = PackHairControlPointPosition(PackedCP, DisplacedPosition);
	}
}

#endif // SHADER_DEFORM_GUIDE