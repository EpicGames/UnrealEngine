// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
float3 DecodeVelocityFromTexture(float4 EncodedV);
#include "HairStrandsVisibilityCommon.ush" 

///////////////////////////////////////////////////////////////////////////////////////////////////
// Used for HitProxy, Selection, and CustomDepth

#if SHADER_EMIT_DEPTH

uint							 bDiscardIfZero;
float 							 CoverageThreshold;
float2							 InvViewportResolution;
uint							 MaxMaterialCount;
Buffer<uint>					 ActiveMaterialIdBuffer;
Texture2D<uint>					 VisNodeIndex;
StructuredBuffer<FPackedHairVis> VisNodeData;
Texture2D<float>				 CoverageTexture;
Texture2D<float>				 HairOnlyDepthTexture;

void EmitPS(
	#if PERMUTATION_TILE
	in FScreenVertexOutput In,
	#else
	in float4 InPosition : SV_POSITION,
	#endif
	#if PERMUTATION_COLOR
	out float4 OutHitProxyId : SV_Target0,
	#endif
	out float OutDepth : SV_Depth)
{
	#if PERMUTATION_COLOR
	OutHitProxyId = 0;
	#endif
	OutDepth = 0;

	bool bDiscard = true;

	#if PERMUTATION_TILE
	const uint2 PixelCoord = (uint2)In.Position.xy;
	const bool bValid = true;
	#else
	const float2 UV = InPosition.xy * InvViewportResolution;
	const uint2 PixelCoord = UV * View.ViewSizeAndInvSize.xy;
	const bool bValid = HairOnlyDepthTexture.Load(uint3(PixelCoord, 0)) > 0;
	#endif

	const FNodeDesc NodeDesc = DecodeNodeDesc(VisNodeIndex.Load(uint3(PixelCoord, 0)));
	const float Coverage = CoverageTexture.Load(uint3(PixelCoord, 0));
	if (bValid && NodeDesc.Count > 0 && Coverage >= CoverageThreshold)
	{
		// Take the first sample
		const uint SampleIndex = NodeDesc.Offset + 0;
		const FHairVis InNode  = UnpackHairVis(VisNodeData[SampleIndex]);
		const uint MaterialId  = clamp(InNode.MaterialId, 0, MaxMaterialCount-1);
		const uint MaterialIdValue = ActiveMaterialIdBuffer[MaterialId];

		#if PERMUTATION_COLOR
		OutHitProxyId = float4(
			float((MaterialIdValue >> 16) & 0xFF) / 255.0f,
			float((MaterialIdValue >> 8) & 0xFF) / 255.0f,
			float((MaterialIdValue >> 0) & 0xFF) / 255.0f,
			0);
		#endif
		OutDepth = InNode.Depth;

		bDiscard = bDiscardIfZero > 0 ? MaterialIdValue == 0 : false;
	}

	if (bDiscard)
	{
		discard;
	}
}

#endif // SHADER_EMIT_DEPTH
