// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#if defined(DXC_GROUPSHARED_ALIGNMENT_WORKAROUND) && DXC_GROUPSHARED_ALIGNMENT_WORKAROUND
// Workaround for shader compiler bug affecting certain platforms
#define NEED_HAIR_MESH_BASIS_PADDING 1
#endif
#include "HairStrandsBindingCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsAABBCommon.ush" 
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

#if COMPILER_METAL || COMPILER_VULKAN
#define ALLOW_DEBUG_RENDERING 0
#else
#define ALLOW_DEBUG_RENDERING 1
#endif

#define DEBUG_ENABLE 0

#if DEBUG_ENABLE && ALLOW_DEBUG_RENDERING
#include "../ShaderPrint.ush"
#include "../ColorMap.ush"
#endif

#pragma warning (default:7203) // downgrade 'not enough registers available for the entire program.' to a warning instead of error


// Deformer types
#define DEFORMER_NONE 0
#define DEFORMER_ADDITIVE 1
#define DEFORMER_OVERWRITE 2

///////////////////////////////////////////////////////////////////////////////////////////////////
// Guides

#if SHADER_HAIRINTERPOLATION || SHADER_PATCHATTRIBUTE
ByteAddressBuffer CurveInterpolationBuffer;
ByteAddressBuffer PointInterpolationBuffer;
#endif

struct FGuideDataWithOffset
{
	uint2  RootPointIndices;
	uint2  PointIndices;
	float2 PointLerps;
	float2 PointWeights;
	float3 OutPositionOffset;
};

FGuideDataWithOffset GetGuideData(FGuideCurveData InCurveData, FGuidePointData InPointData, float3 OutPositionOffset)
{
#if PERMUTATION_SIMULATION > 0
	FGuideDataWithOffset Out;
	Out.RootPointIndices = InCurveData.RootPointIndices;
	Out.PointIndices     = InCurveData.RootPointIndices + InPointData.PointIndices;
	Out.PointLerps       = InPointData.PointLerps;
	Out.PointWeights     = InCurveData.CurveWeights;
	Out.OutPositionOffset= OutPositionOffset;
	return Out;
#else
	return (FGuideDataWithOffset)0;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair interpolation 

// Interpolation methods
#define INTERPOLATION_RIGID 0
#define INTERPOLATION_SKINNING_OFFSET 1
#define INTERPOLATION_SKINNING_TRANSLATION 2
#define INTERPOLATION_SKINNING_ROTATION 3
#define INTERPOLATION_SKINNING_TRANSLATION_AND_ROTATION 4

#ifndef GROUP_SIZE
#error GROUP_SIZE needs to be Undefined
#endif

#if SHADER_HAIRINTERPOLATION

uint DispatchCountX;
uint VertexCount;
uint HairDebugMode;
float HairLengthScale;
float3 InRenHairPositionOffset;
float3 InSimHairPositionOffset;
float4x4 LocalToWorldMatrix;
uint bUseSingleGuide;
uint SimPointCount;
uint SimCurveCount;

ByteAddressBuffer RenRestPosePositionBuffer;
ByteAddressBuffer SimRestPosePositionBuffer;
ByteAddressBuffer SimDeformedPositionBuffer;

uint HairStrandsVFTODO_bCullingEnable;
#if PERMUTATION_CULLING == 1
Buffer<uint>	HairStrandsVFTODO_CullingIndirectBuffer; 
Buffer<uint>	HairStrandsVFTODO_CullingIndexBuffer;
#endif

#if PERMUTATION_DEFORMER
ByteAddressBuffer  RenDeformerPositionBuffer;
#endif
RWByteAddressBuffer OutRenDeformedPositionBuffer; 

#if PERMUTATION_DYNAMIC_GEOMETRY >= 1 

// Compact all these buffers into 2 buffers: translation + quaternion
Buffer<float4>  RenRootRestPositionBuffer;
Buffer<float4>  RenRootDeformedPositionBuffer;
Buffer<uint>	RenRootBarycentricBuffer;
Buffer<uint>	RenRootToUniqueTriangleIndexBuffer;

Buffer<float4>  SimRootRestPositionBuffer;
Buffer<float4>  SimRootDeformedPositionBuffer;
Buffer<uint>	SimRootBarycentricBuffer;
Buffer<uint>	SimRootToUniqueTriangleIndexBuffer;
#endif

uint InstanceRegisteredIndex;
StructuredBuffer<float4> OutSimHairPositionOffsetBuffer;
StructuredBuffer<float4> OutRenHairPositionOffsetBuffer;

float3 GetSimPositionOffset() { return ReadSimPositionOffset(OutSimHairPositionOffsetBuffer, InstanceRegisteredIndex); }
float3 GetRenPositionOffset() { return ReadRenPositionOffset(OutRenHairPositionOffsetBuffer, InstanceRegisteredIndex); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Deformation offset

struct FGuideIndex
{
	uint PointIndex;
	uint RootPointIndex;
};

FGuideIndex ToGuideIndex(FGuideDataWithOffset In, uint InK, uint InOffset)
{
	FGuideIndex Out;
	Out.PointIndex		= In.PointIndices[InK] + InOffset;
	Out.RootPointIndex	= In.RootPointIndices[InK];
	return Out;
};


float3 ComputeStaticGeometryOffset(
	FGuideIndex GuideIndex,
	float GuideVertexWeight, 
	float3 OutSimHairPositionOffset)
{
	const float3 RestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.PointIndex, InSimHairPositionOffset);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.PointIndex, OutSimHairPositionOffset);

	return (DeformedGuidePoint - RestGuidePoint) * GuideVertexWeight;
}

float3 ComputeLocalGeometryOffset(
	FGuideIndex GuideIndex, 
	float GuideVertexWeight)
{
	const float3 RestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.RootPointIndex);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.RootPointIndex);

	return (DeformedGuidePoint - RestGuidePoint) * GuideVertexWeight;
}

float3 ComputeDynamicGeometryOffset(
	FGuideIndex GuideIndex, 
	float GuideVertexWeight, 
	FHairMeshBasis RestBasis, 
	FHairMeshBasis DeformedBasis,
	float3 OutSimHairPositionOffset)
{
	const float3 RestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.PointIndex, InSimHairPositionOffset);
	const float3 LocalRestGuidePoint = ToTriangle(RestGuidePoint, RestBasis);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.PointIndex, OutSimHairPositionOffset);
	const float3 LocalDeformedGuidePoint = ToTriangle(DeformedGuidePoint, DeformedBasis);

	return (LocalDeformedGuidePoint - LocalRestGuidePoint) * GuideVertexWeight;
}

float3 ComputeTranslateGeometryOffset(
	FGuideIndex GuideIndex, 
	float GuideVertexWeight)
{
	const float3 RestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.RootPointIndex);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.RootPointIndex);

	return (DeformedGuidePoint - RestGuidePoint) * GuideVertexWeight;
}

float3 ComputeRotateGeometryOffset(
	FGuideIndex GuideIndex, 
	float GuideVertexWeight,
	FHairMeshBasis RestBasis, 
	FHairMeshBasis DeformedBasis)
{	
	const float3 RestGuidePoint = ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimRestPosePositionBuffer, GuideIndex.RootPointIndex);
	const float3 LocalRestGuidePoint = VectorToTriangle(RestGuidePoint, RestBasis);

	const float3 DeformedGuidePoint = ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.PointIndex) - ReadHairControlPointPosition(SimDeformedPositionBuffer, GuideIndex.RootPointIndex);
	const float3 LocalDeformedGuidePoint = VectorToTriangle(DeformedGuidePoint, DeformedBasis);

	return (LocalDeformedGuidePoint - LocalRestGuidePoint) * GuideVertexWeight;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Root triangles

struct FRootBasis
{
	FHairMeshBasis RestBasis;
	FHairMeshBasis DeformedBasis;
};

FRootBasis GetRootBasis(uint CurveIndex, bool bSim)
{
	FRootBasis Out = (FRootBasis)0;
	#if PERMUTATION_DYNAMIC_GEOMETRY >= 1 
	if (bSim)
	{
		const uint TriangleIndex = SimRootToUniqueTriangleIndexBuffer[CurveIndex];
		const float3 RootBarycentric  = UnpackBarycentrics(SimRootBarycentricBuffer[CurveIndex]);
		Out.RestBasis	  = GetHairMeshBasis(TriangleIndex, SimRootRestPositionBuffer, RootBarycentric);
		Out.DeformedBasis = GetHairMeshBasis(TriangleIndex, SimRootDeformedPositionBuffer, RootBarycentric);
	}
	else
	{
		const uint TriangleIndex = RenRootToUniqueTriangleIndexBuffer[CurveIndex];
		const float3 RootBarycentric  = UnpackBarycentrics(RenRootBarycentricBuffer[CurveIndex]);
		Out.RestBasis	  = GetHairMeshBasis(TriangleIndex, RenRootRestPositionBuffer, RootBarycentric);
		Out.DeformedBasis = GetHairMeshBasis(TriangleIndex, RenRootDeformedPositionBuffer, RootBarycentric);
	}
	#endif
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Rigid interpolation
float3 ApplyRigidInterpolation(FGuideDataWithOffset GuideData, float3 InPosition)
{
	float3 CurrOffset = 0;
#if PERMUTATION_SIMULATION == 1
	#if PERMUTATION_USE_SINGLE_GUIDE == 1
	const uint K = 0;
	#else
	[unroll]
	for (uint K = 0; K < HAIR_INTERPOLATION_MAX_GUIDE_COUNT; ++K)
	#endif
	{
		const float3 Offset0 = ComputeStaticGeometryOffset(ToGuideIndex(GuideData, K, 0), GuideData.PointWeights[K], GuideData.OutPositionOffset);
		const float3 Offset1 = ComputeStaticGeometryOffset(ToGuideIndex(GuideData, K, 1), GuideData.PointWeights[K], GuideData.OutPositionOffset);

		CurrOffset += lerp(Offset0, Offset1, GuideData.PointLerps[K]);
	}
#endif

	return InPosition + CurrOffset;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Skinning with offset
float3 ApplySkinningWithOffset(const FGuideDataWithOffset GuideData, const FRootBasis RenTri, const FRootBasis SimTris[2], float3 InPosition)
{
	float3 CurrOffset = 0;

	// Compute the simulation offset in hair local space (i.e., triangle)
#if PERMUTATION_SIMULATION == 1
	#if PERMUTATION_USE_SINGLE_GUIDE == 1
	const uint K = 0;
	#else
	[unroll]
	for (uint K = 0; K < HAIR_INTERPOLATION_MAX_GUIDE_COUNT; ++K)
	#endif
	{
		const FRootBasis SimTri = SimTris[K];

		const float3 Offset0 = ComputeDynamicGeometryOffset(ToGuideIndex(GuideData, K, 0), GuideData.PointWeights[K], SimTri.RestBasis, SimTri.DeformedBasis, GuideData.OutPositionOffset);
		const float3 Offset1 = ComputeDynamicGeometryOffset(ToGuideIndex(GuideData, K, 1), GuideData.PointWeights[K], SimTri.RestBasis, SimTri.DeformedBasis, GuideData.OutPositionOffset);

		CurrOffset += VectorToWorld(lerp(Offset0, Offset1, GuideData.PointLerps[K]), SimTri.DeformedBasis);
	}
#endif		

	InPosition = ToTriangle(InPosition, RenTri.RestBasis);
	InPosition = ToWorld(InPosition, RenTri.DeformedBasis) + CurrOffset;

	return InPosition;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Skinning with translantion
float3 ApplySkinningWithTranslation(const FGuideDataWithOffset GuideData, const FRootBasis RenTri, float3 InPosition)
{
	InPosition -= RenTri.RestBasis.BaseO;

	float3 CurrOffset = 0;

#if PERMUTATION_SIMULATION == 1
#if PERMUTATION_USE_SINGLE_GUIDE == 1
	const uint K = 0;
	#else
	[unroll]
	for (uint K = 0; K < HAIR_INTERPOLATION_MAX_GUIDE_COUNT; ++K)
	#endif
	{
		const float3 Offset0 = ComputeTranslateGeometryOffset(ToGuideIndex(GuideData, K, 0), 1.0);
		const float3 Offset1 = ComputeTranslateGeometryOffset(ToGuideIndex(GuideData, K, 1), 1.0);

		CurrOffset += GuideData.PointWeights[K] * lerp(Offset0, Offset1, GuideData.PointLerps[K]);
	}
#endif

	InPosition += RenTri.DeformedBasis.BaseO;
	return InPosition + CurrOffset;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Skinning with rotation
float3 ApplySkinningWithRotation(const FGuideDataWithOffset GuideData, const FRootBasis RenTri, const FRootBasis SimTris[2], float3 InPosition)
{
	InPosition -= RenTri.RestBasis.BaseO;

	float3 CurrOffset = 0;

#if PERMUTATION_SIMULATION == 1
	#if PERMUTATION_USE_SINGLE_GUIDE == 1
	const uint K = 0;
	#else
	[unroll]
	for (uint K = 0; K < HAIR_INTERPOLATION_MAX_GUIDE_COUNT; ++K)
	#endif
	{
		const FRootBasis SimTri = SimTris[K];

		const float3 LocalPoint = VectorToTriangle(InPosition, SimTri.RestBasis);

		const float3 Offset0 = ComputeRotateGeometryOffset(ToGuideIndex(GuideData, K, 0), 1.0, SimTri.RestBasis, SimTri.DeformedBasis/*, SimHairPositionOffset*/);
		const float3 Offset1 = ComputeRotateGeometryOffset(ToGuideIndex(GuideData, K, 1), 1.0, SimTri.RestBasis, SimTri.DeformedBasis/*, SimHairPositionOffset*/);

		CurrOffset += GuideData.PointWeights[K] * VectorToWorld(lerp(Offset0, Offset1, GuideData.PointLerps[K]) + LocalPoint, SimTri.DeformedBasis);
	}
#endif

	InPosition = RenTri.DeformedBasis.BaseO;
	return InPosition + CurrOffset;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Skinning with translation and rotation
float3 ApplySkinningWithTranslationAndRotation(const FGuideDataWithOffset GuideData, const FRootBasis RenTri, const FRootBasis SimTris[2], float3 InPosition)
{
	float3 CurrOffset = 0;

#if PERMUTATION_SIMULATION == 1
	#if PERMUTATION_USE_SINGLE_GUIDE == 1
	const uint K = 0;
	#else
	[unroll]
	for (uint K = 0; K < HAIR_INTERPOLATION_MAX_GUIDE_COUNT; ++K)
	#endif
	{
		const FRootBasis SimTri = SimTris[K];  //GetRootTriangleData(GuideIndex0, true);

		const float3 LocalPoint = ToTriangle(InPosition, SimTri.RestBasis);

		const float3 Offset0 = ComputeDynamicGeometryOffset(ToGuideIndex(GuideData, K, 0), 1.0, SimTri.RestBasis, SimTri.DeformedBasis, GuideData.OutPositionOffset);
		const float3 Offset1 = ComputeDynamicGeometryOffset(ToGuideIndex(GuideData, K, 1), 1.0, SimTri.RestBasis, SimTri.DeformedBasis, GuideData.OutPositionOffset);

		CurrOffset += GuideData.PointWeights[K] * ToWorld(lerp(Offset0, Offset1, GuideData.PointLerps[K]) + LocalPoint, SimTri.DeformedBasis);
	}
#endif

	return CurrOffset;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Deformer transformation
float3 ApplyDeformer(const FRootBasis RenTri, float3 InRestPostion, float3 InDeformerPosition, float3 InDeformedPosition)
{
	float3 OutPosition = InDeformedPosition;
#if PERMUTATION_DEFORMER
	#if PERMUTATION_DYNAMIC_GEOMETRY == 0
		OutPosition = InDeformedPosition + (InDeformerPosition - InRestPostion);
	#else
		const float3 LocalRestPoint     = ToTriangle(InRestPostion, RenTri.RestBasis);
		const float3 LocalDeformerPoint = ToTriangle(InDeformerPosition, RenTri.RestBasis);

		const float3 LocalOffset = LocalDeformerPoint - LocalRestPoint;
		const float3 LocalDeformedPoint = ToTriangle(InDeformedPosition, RenTri.DeformedBasis);
		OutPosition = ToWorld(LocalDeformedPoint + LocalOffset, RenTri.DeformedBasis);
	#endif
#endif
	return OutPosition;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Interpolation

uint CurveCount;

ByteAddressBuffer RenCurveBuffer;

#define TOTALMAXPOINTPERCURVE 32
//-------------------------------------------------------------------------------------------------

#define CURVE_PER_GROUP (GROUP_SIZE / PERMUTATION_POINT_PER_CURVE)
#define MAX_CURVE CURVE_PER_GROUP
#define MAX_POINT PERMUTATION_POINT_PER_CURVE

uint GetGlobalCurveIndex(uint ThreadIndex, uint2 GroupIndex)
{
	const uint LinearGroupIndex = GroupIndex.x + GroupIndex.y * DispatchCountX;
#if PERMUTATION_POINT_PER_CURVE == TOTALMAXPOINTPERCURVE
	return LinearGroupIndex;
	#else
	return ThreadIndex + LinearGroupIndex * CURVE_PER_GROUP;
	#endif
}

//-------------------------------------------------------------------------------------------------
#if PERMUTATION_POINT_PER_CURVE < TOTALMAXPOINTPERCURVE

groupshared uint CurvePointIndex[MAX_CURVE];
groupshared uint CurvePointCount[MAX_CURVE];

groupshared FRootBasis g_RenTri[MAX_CURVE];
groupshared float3 RenPositionOffset;

groupshared FGuideCurveData SimCurveData[MAX_CURVE];
groupshared FRootBasis g_SimTris[MAX_CURVE][2];

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint ThreadIndex : SV_GroupIndex, uint2 GroupIndex : SV_GroupID)
{
	// 1. Read the Curve data (1 thread per curve)
	if (ThreadIndex < CURVE_PER_GROUP)
	{
		const uint LocalCurveIndex  = ThreadIndex;
		const uint GlobalCurveIndex = GetGlobalCurveIndex(ThreadIndex, GroupIndex);
		const bool bValidCurveIndex = GlobalCurveIndex < CurveCount;

		CurvePointIndex[LocalCurveIndex] = 0;
		CurvePointCount[LocalCurveIndex] = 0;
		if (bValidCurveIndex)
		{
			// Curve's point count/offset
			const FHairCurve Curve = ReadHairCurve(RenCurveBuffer, GlobalCurveIndex);
			CurvePointIndex[LocalCurveIndex] = Curve.PointIndex;
			CurvePointCount[LocalCurveIndex] = Curve.PointCount;	

			// Load curve interpolation data
			#if PERMUTATION_SIMULATION
			SimCurveData[LocalCurveIndex] = UnpackGuideCurveData(CurveInterpolationBuffer, GlobalCurveIndex, bUseSingleGuide);
			if (PERMUTATION_USE_SINGLE_GUIDE)
			{
				SimCurveData[LocalCurveIndex].CurveWeights.x = 1;
				SimCurveData[LocalCurveIndex].CurveWeights.y = 0;
			}
			#else
			SimCurveData[LocalCurveIndex] = (FGuideCurveData)0;
			#endif

			// Load ren triangles
			g_RenTri[LocalCurveIndex] = GetRootBasis(GlobalCurveIndex, false/*bSim*/);

			// Load sim triangles
			g_SimTris[LocalCurveIndex][0] = (FRootBasis)0;
			g_SimTris[LocalCurveIndex][1] = (FRootBasis)0;
			#if PERMUTATION_SIMULATION
			g_SimTris[LocalCurveIndex][0] = GetRootBasis(SimCurveData[LocalCurveIndex].CurveIndices[0], true/*bSim*/);
			if (!PERMUTATION_USE_SINGLE_GUIDE)
			{
				g_SimTris[LocalCurveIndex][1] = GetRootBasis(SimCurveData[LocalCurveIndex].CurveIndices[1], true/*bSim*/);
			}
			#endif
		}
	}
	if (ThreadIndex == 0)
	{
		RenPositionOffset = GetRenPositionOffset();
	}
	GroupMemoryBarrierWithGroupSync();
	
	// 2. Apply deformation (1 thread per point)
	{
		const uint LocalCurveIndex   = ThreadIndex / uint(MAX_POINT);
		const uint CurrentPointIndex = ThreadIndex % uint(MAX_POINT);

		// Use LocalCurveIndex to compute the global curve index.
		// Here 1thread == 1control point, while in the previous block it was 1thread == 1curve. 
		// This is to remaps correctly the curve index.
		const uint GlobalCurveIndex = GetGlobalCurveIndex(LocalCurveIndex, GroupIndex); 
		if (GlobalCurveIndex < CurveCount)
		{		
			if (CurrentPointIndex < CurvePointCount[LocalCurveIndex])
			{
				const uint VertexIndex = CurrentPointIndex + CurvePointIndex[LocalCurveIndex];

				#if PERMUTATION_SIMULATION
				const FGuidePointData SimPointData = UnpackGuidePointData(PointInterpolationBuffer, VertexIndex, bUseSingleGuide);
				const FGuideDataWithOffset GuideData = GetGuideData(SimCurveData[LocalCurveIndex], SimPointData, GetSimPositionOffset());
				#else
				const FGuidePointData SimPointData = (FGuidePointData)0;
				const FGuideDataWithOffset GuideData = (FGuideDataWithOffset)0;;
				#endif

				// Remap from LDS to local variable to ease the setup
				FRootBasis RenTri = g_RenTri[LocalCurveIndex];
				FRootBasis SimTris[2];
				#if PERMUTATION_SIMULATION
				SimTris[0] = g_SimTris[LocalCurveIndex][0];
				SimTris[1] = g_SimTris[LocalCurveIndex][1];
				#else
				SimTris[0] = (FRootBasis)0;
				SimTris[1] = (FRootBasis)0;
				#endif

				// Manual decoding of the rest position
				FPackedHairPosition PackedRestPosition = ReadPackedHairPosition(RenRestPosePositionBuffer, VertexIndex);
				const float3 RestPosition = UnpackHairControlPointPosition(PackedRestPosition, InRenHairPositionOffset);
				
				float3 OutPosition = 0.0f;

				// 1. Rigid transformation
				// ControlPoint is in the local hair referential
				// CurrOffset takes only translation component into account, and is done in object local space (vs. triangle/hair local space)
				#if PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_RIGID
				{
					OutPosition = ApplyRigidInterpolation(GuideData, RestPosition);
				}
				// 2. Skin transformation
				// Apply dynamic mesh deformation (translation / rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_OFFSET
				{
					OutPosition = ApplySkinningWithOffset(GuideData, RenTri, SimTris, RestPosition);
				}
				// 3. Linear blend skinning (translation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_TRANSLATION
				{
					OutPosition = ApplySkinningWithTranslation(GuideData, RenTri, RestPosition);
				}
				// 4. Linear blend skinning (rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_ROTATION
				{
					OutPosition = ApplySkinningWithRotation(GuideData, RenTri, SimTris, RestPosition);
				}
				// 5. Linear blend skinning (translation + rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_TRANSLATION_AND_ROTATION
				{
					OutPosition = ApplySkinningWithTranslationAndRotation(GuideData, RenTri, SimTris, RestPosition);
				}
				#endif

				// 3. Apply deformer
				#if PERMUTATION_DEFORMER == DEFORMER_ADDITIVE
				{
					const FPackedHairPosition PackedDeformerPosition = ReadPackedHairPosition(RenDeformerPositionBuffer, VertexIndex);
					const float3 DeformerPosition = UnpackHairControlPointPosition(PackedDeformerPosition, InRenHairPositionOffset);
					OutPosition = ApplyDeformer(RenTri, RestPosition, DeformerPosition, OutPosition);
					CopyPackedAttribute(PackedRestPosition, PackedDeformerPosition);
				}
				#elif PERMUTATION_DEFORMER == DEFORMER_OVERWRITE
				{
					const FPackedHairPosition PackedDeformerPosition = ReadPackedHairPosition(RenDeformerPositionBuffer, VertexIndex);
					OutPosition = UnpackHairControlPointPosition(PackedDeformerPosition, InRenHairPositionOffset);
					CopyPackedAttribute(PackedRestPosition, PackedDeformerPosition);
				}
				// 3. Linear blend skinning (translation)
				#endif

				// 4. Write out the final position
				// Optionally trim hair
				if (HairLengthScale < 1.0f)
				{
					const float CoordU = UnpackHairControlPointCoordU(PackedRestPosition);
					if (HairLengthScale < CoordU)
					{
						OutPosition = INFINITE_FLOAT;
					}
				}

				WritePackedHairControlPointPosition(OutRenDeformedPositionBuffer, VertexIndex, PackedRestPosition, OutPosition, RenPositionOffset);
			}
		}
	}
}
#endif // PERMUTATION_POINT_PER_CURVE < TOTALMAXPOINTPERCURVE
//-------------------------------------------------------------------------------------------------
#if PERMUTATION_POINT_PER_CURVE == TOTALMAXPOINTPERCURVE

#if PERMUTATION_WAVEOPS
FRootBasis WaveReadLaneFirstTriangle(FRootBasis In)
{
	FRootBasis Out;
	Out.RestBasis.BaseO = WaveReadLaneFirst(In.RestBasis.BaseO);
	Out.RestBasis.BaseT = WaveReadLaneFirst(In.RestBasis.BaseT);
	Out.RestBasis.BaseB = WaveReadLaneFirst(In.RestBasis.BaseB);
	Out.RestBasis.BaseN = WaveReadLaneFirst(In.RestBasis.BaseN);

	Out.DeformedBasis.BaseO = WaveReadLaneFirst(In.DeformedBasis.BaseO);
	Out.DeformedBasis.BaseT = WaveReadLaneFirst(In.DeformedBasis.BaseT);
	Out.DeformedBasis.BaseB = WaveReadLaneFirst(In.DeformedBasis.BaseB);
	Out.DeformedBasis.BaseN = WaveReadLaneFirst(In.DeformedBasis.BaseN);

	return Out;
}

FGuideCurveData WaveReadLaneFirstCurveData(FGuideCurveData In)
{
	FGuideCurveData Out;
	Out.CurveIndices	 = WaveReadLaneFirst(In.CurveIndices);
	Out.CurveWeights	 = WaveReadLaneFirst(In.CurveWeights);
	Out.RootPointIndices = WaveReadLaneFirst(In.RootPointIndices);
	return Out;
}
#else
groupshared uint CurvePointIndex;
groupshared uint CurvePointCount;

groupshared FRootBasis RenTri;
groupshared float3 RenPositionOffset;

groupshared FGuideCurveData SimCurveData;
groupshared FRootBasis SimTris[2];
#endif

#if PERMUTATION_WAVEOPS && COMPILER_SUPPORTS_WAVE_SIZE
WAVESIZE(32) // PERMUTATION_WAVE_OPS is true only when wave>=32 are available
#endif
[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint ThreadIndex : SV_GroupIndex, uint2 GroupIndex : SV_GroupID)
{
	// 1. Read the Curve data
	const uint CurveIndex = GetGlobalCurveIndex(ThreadIndex, GroupIndex);
	const bool bIsCurveValid = CurveIndex < CurveCount;
	if (!bIsCurveValid)
	{
		return;
	}
	
	#if PERMUTATION_WAVEOPS
	// 2. Curve's point count/offset
	uint CurvePointIndex = 0;
	uint CurvePointCount = 0;

	float3 RenPositionOffset = 0;
	FRootBasis RenTri = (FRootBasis)0;

	FGuideCurveData SimCurveData = (FGuideCurveData)0;
	FRootBasis SimTris[2];
	SimTris[0] = (FRootBasis)0;
	SimTris[1] = (FRootBasis)0;
	#endif

	// 2. Curve's point count/offset
	#if PERMUTATION_WAVEOPS
	if (WaveIsFirstLane())
	#else
	if (ThreadIndex == 0)
	#endif
	{
		CurvePointIndex = 0;
		CurvePointCount = 0;
		if (bIsCurveValid)
		{
			const FHairCurve Curve = ReadHairCurve(RenCurveBuffer, CurveIndex);
			CurvePointIndex = Curve.PointIndex;
			CurvePointCount = Curve.PointCount;

			// Load curve interpolation data
			#if PERMUTATION_SIMULATION
			SimCurveData = UnpackGuideCurveData(CurveInterpolationBuffer, CurveIndex, bUseSingleGuide);
			if (PERMUTATION_USE_SINGLE_GUIDE)
			{
				SimCurveData.CurveWeights.x = 1;
				SimCurveData.CurveWeights.y = 0;
			}
			#else
			SimCurveData = (FGuideCurveData)0;
			#endif

			// Load sim/ren triangles
			RenTri = GetRootBasis(CurveIndex, false/*bSim*/);

			#if PERMUTATION_SIMULATION
			SimTris[0] = GetRootBasis(SimCurveData.CurveIndices[0], true/*bSim*/);
			SimTris[1] = SimTris[0];
			if (!PERMUTATION_USE_SINGLE_GUIDE)
			{
				SimTris[1] = GetRootBasis(SimCurveData.CurveIndices[1], true/*bSim*/);
			}
			#else
			SimTris[0] = (FRootBasis)0;
			SimTris[1] = (FRootBasis)0;
			#endif
		}
		RenPositionOffset = GetRenPositionOffset();
	}

	#if PERMUTATION_WAVEOPS
	{
		// Broadcast values
		CurvePointIndex   = WaveReadLaneFirst(CurvePointIndex);
		CurvePointCount   = WaveReadLaneFirst(CurvePointCount);
		RenPositionOffset = WaveReadLaneFirst(RenPositionOffset);
		#if PERMUTATION_DYNAMIC_GEOMETRY >= 1
		RenTri = WaveReadLaneFirstTriangle(RenTri);
		#endif

		#if PERMUTATION_SIMULATION
		SimCurveData = WaveReadLaneFirstCurveData(SimCurveData);
		SimTris[0] = WaveReadLaneFirstTriangle(SimTris[0]);
		if (!PERMUTATION_USE_SINGLE_GUIDE)
		{
			SimTris[1] = WaveReadLaneFirstTriangle(SimTris[1]);
		}		
		#endif
	}
	#else
	GroupMemoryBarrierWithGroupSync();
	#endif

	// 2. Apply deformation
	{
		const uint LoopCount = DivideAndRoundUp(CurvePointCount, GROUP_SIZE);
		for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
		{		
			const uint CurrentPointIndex = ThreadIndex + LoopIt * GROUP_SIZE;
	
			if (CurrentPointIndex < CurvePointCount)
			{
				const uint VertexIndex = CurrentPointIndex + CurvePointIndex;

				#if PERMUTATION_SIMULATION
				const FGuidePointData SimPointData = UnpackGuidePointData(PointInterpolationBuffer, VertexIndex, bUseSingleGuide);
				const FGuideDataWithOffset GuideData = GetGuideData(SimCurveData, SimPointData, GetSimPositionOffset());
				#else
				const FGuidePointData SimPointData = (FGuidePointData)0;
				const FGuideDataWithOffset GuideData = (FGuideDataWithOffset)0;;
				#endif

				// Manual decoding of the rest position
				FPackedHairPosition PackedRestPosition = ReadPackedHairPosition(RenRestPosePositionBuffer, VertexIndex);
				const float3 RestPosition = UnpackHairControlPointPosition(PackedRestPosition, InRenHairPositionOffset);

				float3 OutPosition = 0.0f;

				// 1. Rigid transformation
				// ControlPoint is in the local hair referential
				// CurrOffset takes only translation component into account, and is done in object local space (vs. triangle/hair local space)
				#if PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_RIGID
				{
					OutPosition = ApplyRigidInterpolation(GuideData, RestPosition);
				}
				// 2. Skin transformation
				// Apply dynamic mesh deformation (translation / rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_OFFSET
				{
					OutPosition = ApplySkinningWithOffset(GuideData, RenTri, SimTris, RestPosition);
				}
				// 3. Linear blend skinning (translation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_TRANSLATION
				{
					OutPosition = ApplySkinningWithTranslation(GuideData, RenTri, RestPosition);
				}
				// 4. Linear blend skinning (rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_ROTATION
				{
					OutPosition = ApplySkinningWithRotation(GuideData, RenTri, SimTris, RestPosition);
				}
				// 5. Linear blend skinning (translation + rotation)
				#elif PERMUTATION_DYNAMIC_GEOMETRY == INTERPOLATION_SKINNING_TRANSLATION_AND_ROTATION
				{
					OutPosition = ApplySkinningWithTranslationAndRotation(GuideData, RenTri, SimTris, RestPosition);
				}
				#endif

				// 3. Apply deformer
				#if PERMUTATION_DEFORMER == DEFORMER_ADDITIVE
				{
					const FPackedHairPosition PackedDeformerPosition = ReadPackedHairPosition(RenDeformerPositionBuffer, VertexIndex);
					const float3 DeformerPosition = UnpackHairControlPointPosition(PackedDeformerPosition, InRenHairPositionOffset);
					OutPosition = ApplyDeformer(RenTri, RestPosition, DeformerPosition, OutPosition);
					CopyPackedAttribute(PackedRestPosition, PackedDeformerPosition);
				}
				#elif PERMUTATION_DEFORMER == DEFORMER_OVERWRITE
				{
					const FPackedHairPosition PackedDeformerPosition = ReadPackedHairPosition(RenDeformerPositionBuffer, VertexIndex);
					OutPosition = UnpackHairControlPointPosition(PackedDeformerPosition, InRenHairPositionOffset);
					CopyPackedAttribute(PackedRestPosition, PackedDeformerPosition);
				}
				#endif

				// 4. Write out the final position
				// Optionally trim hair
				if (HairLengthScale < 1.0f)
				{
					const float CoordU = UnpackHairControlPointCoordU(PackedRestPosition);
					if (HairLengthScale < CoordU)
					{
						OutPosition = INFINITE_FLOAT;
					}
				}

				WritePackedHairControlPointPosition(OutRenDeformedPositionBuffer, VertexIndex, PackedRestPosition, OutPosition, RenPositionOffset);
			}
		}
	}
}
#endif //PERMUTATION_POINT_PER_CURVE == TOTALMAXPOINTPERCURVE
#endif // SHADER_HAIRINTERPOLATION

///////////////////////////////////////////////////////////////////////////////////////////////////
// Patch attribute (for debug visualization)
#if SHADER_PATCHATTRIBUTE

#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsAttributeCommon.ush"

uint				CurveCount;
uint				bUseSingleGuide;

ByteAddressBuffer	RenCurveBuffer;
Buffer<uint>		RenCurveToClusterIdBuffer;
RWByteAddressBuffer	OutRenAttributeBuffer;

uint 				CurveAttributeIndexToChunkDivAsShift;
uint 				CurveAttributeChunkElementCount;
uint 				CurveAttributeChunkStrideInBytes;
uint 				PointAttributeIndexToChunkDivAsShift;
uint 				PointAttributeChunkElementCount;
uint 				PointAttributeChunkStrideInBytes;
uint4				CurveAttributeOffsets[HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT];

groupshared uint SeedValues[1024];

uint ToSeedHash(uint In)
{
	const uint InLo8bits =  In & 0xFF;
	const uint InHi8btis = (In >> 8) & 0xFF;

	// Using FNV1 hash to break linearity of ClusterId (generated by linearly parsing cluster grid)
	uint Hash = 0;
	Hash = Hash * 17;
	Hash = Hash ^ InLo8bits;
	Hash = Hash * 17;
	Hash = Hash ^ InHi8btis;
	return (Hash & 0xFF);
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint GroupThread1D : SV_GroupIndex)
{
	SeedValues[GroupThread1D] = 0;

	const uint CurveIndex = DispatchThreadId.x;
	const bool bValid = CurveIndex < CurveCount;
	if (bValid)
	{
		uint Out = 0;
		#if PERMUTATION_SIMULATION == 0
		{
			const uint ClusterId		= RenCurveToClusterIdBuffer[CurveIndex];
			SeedValues[GroupThread1D]	= ToSeedHash(ClusterId);
		}
		#else // if (PERMUTATION_SIMULATION > 0)
		{

			const uint VertexIndex0 = ReadHairCurve(RenCurveBuffer, CurveIndex).PointIndex;

			const FGuideCurveData SimCurveData = UnpackGuideCurveData(CurveInterpolationBuffer, CurveIndex, bUseSingleGuide);
			const FGuidePointData SimPointData = UnpackGuidePointData(PointInterpolationBuffer, VertexIndex0, bUseSingleGuide); 
			const FGuideDataWithOffset GuideData = GetGuideData(SimCurveData, SimPointData, 0 /*OutPositionOffset*/);

			const uint GuideIndex		= GuideData.PointIndices[0]; // Take the closest guide
			SeedValues[GroupThread1D]	= ToSeedHash(GuideIndex);
		}
		#endif // PERMUTATION_SIMULATION
	}

	GroupMemoryBarrierWithGroupSync();

	// Write 4 values at a type since seed are encoded into 8bits, and data are written out as uint
	if ((CurveIndex % 4) == 0)
	{	
		uint Out = 0;
		Out |= (SeedValues[GroupThread1D]   & 0xFF);
		Out |= (SeedValues[GroupThread1D+1] & 0xFF) << 8;
		Out |= (SeedValues[GroupThread1D+2] & 0xFF) << 16;
		Out |= (SeedValues[GroupThread1D+3] & 0xFF) << 24;

		uint WriteIndex 				= CurveIndex;
		uint WriteAttributOffsetInBytes = HAIR_CURVE_ATTRIBUTE_OFFSET_SEED(CurveAttributeOffsets);

		const uint ChunkIndex   	= CurveIndex >> CurveAttributeIndexToChunkDivAsShift;
		WriteIndex                 -= ChunkIndex * CurveAttributeChunkElementCount;
		WriteAttributOffsetInBytes += ChunkIndex * CurveAttributeChunkStrideInBytes;

		OutRenAttributeBuffer.Store(WriteAttributOffsetInBytes + WriteIndex, Out);
	}
}
#endif // SHADER_PATCHATTRIBUTE

///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair transfer prev. position

#if SHADER_HAIRTRANSFER_PREV_POSITION

uint PointCount;
uint PointOffset;
uint TotalPointCount;
ByteAddressBuffer InBuffer;
RWByteAddressBuffer OutBuffer;

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint PointIndex = DispatchThreadId + PointOffset;
	if (DispatchThreadId < PointCount && PointIndex < TotalPointCount)
	{
		CopyPackedHairControlPoint(OutBuffer, InBuffer, PointIndex);
	}
}

#endif // SHADER_HAIRTRANSFER_PREV_POSITION

///////////////////////////////////////////////////////////////////////////////////////////////////
// Groom cache update

#if SHADER_GROOMCACHE_UPDATE

uint InstanceRegisteredIndex;
uint CachePointCount;
uint PointCount;
uint CurveCount;
uint bHasRadiusData;
uint bHasAddedControlPoint;
float InterpolationFactor;
float MaxHairRadius;

Buffer<float> InPosition0Buffer;
Buffer<float> InPosition1Buffer;
Buffer<float> InRadius0Buffer;
Buffer<float> InRadius1Buffer;

ByteAddressBuffer InRestCurveBuffer;
ByteAddressBuffer InRestPositionBuffer;
ByteAddressBuffer InRestPointToCurveBuffer;
StructuredBuffer<float4> InDeformedOffsetBuffer;
RWByteAddressBuffer OutDeformedBuffer;

float3 ReadCachePosition(uint InCachePointIndex)
{
	const uint Index3 = InCachePointIndex * 3;
	const float3 InPosition0 = float3(
		InPosition0Buffer[Index3],
		InPosition0Buffer[Index3+1],
		InPosition0Buffer[Index3+2]);
	
	const float3 InPosition1 = float3(
		InPosition1Buffer[Index3],
		InPosition1Buffer[Index3+1],
		InPosition1Buffer[Index3+2]);
	return lerp(InPosition0, InPosition1, InterpolationFactor);
}

float ReadCacheRadius(uint InCachePointIndex)
{
	return lerp(InRadius0Buffer[InCachePointIndex], InRadius1Buffer[InCachePointIndex], InterpolationFactor);
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint RestPointIndex = DispatchThreadId;
	if (RestPointIndex >= PointCount)
	{
		return;
	}

	const uint CurveIndex = ReadHairPointToCurveIndex(InRestPointToCurveBuffer, RestPointIndex);
	const FHairCurve Curve = ReadHairCurve(InRestCurveBuffer, CurveIndex);
	const float3 DeformedPositionOffset = ReadRenPositionOffset(InDeformedOffsetBuffer, InstanceRegisteredIndex);

	// Cache data does not contain extra CP at the end of each curve when UseTriangleStrip geometry is used.
	// This is because the cache data are directly serialized in to the asset, and needs to be independent of this project settings.
	// To account for this, we remap the rest position to the cache data. The added CP is set to the last point curve point available in the cache. 
	// This ensures that the guide cache and the strands cache produce valid results
	uint CachePointIndex = CachePointCount;
	bool bExtendedPosition = false;
	if (bHasAddedControlPoint)
	{
		// Make the assumption that there is no clamped curve data.
		// One extra point per curve.
		const bool bIsEndPoint = RestPointIndex == (Curve.PointIndex + Curve.PointCount - 1);
		if (bIsEndPoint)
		{
			const uint NumAddedPoints  = CurveIndex + 1; // 1 CP for each curve + 1 CP for this last point
			CachePointIndex = RestPointIndex - NumAddedPoints;
			bExtendedPosition = Curve.PointCount > 1 && CachePointIndex > 0; // If this is the last point, we need to shift it slightly so that normal computation are correct, otherwise the shader of the last visible CP will be incorrect
		}
		else
		{
			const uint NumAddedPoints = CurveIndex; // 1 CP for each curve
			CachePointIndex = RestPointIndex - NumAddedPoints;
		}
	}
	else
	{
		CachePointIndex = RestPointIndex;
	}

	if (CachePointIndex < CachePointCount)
	{
		FHairControlPoint CP = ReadHairControlPoint(InRestPositionBuffer, RestPointIndex, float3(0, 0, 0)/*Offset*/, 1/*Radius*/, 1/*RootScale*/, 1/*TipScale*/);

		// Position
		if (bExtendedPosition)
		{
			const int CachePointIndex0 = CachePointIndex-1;
			const int CachePointIndex1 = CachePointIndex;
			float3 P0 = ReadCachePosition(CachePointIndex0);
			float3 P1 = ReadCachePosition(CachePointIndex1);
			CP.Position = (P1 - P0) * 0.1f + P1;
		}
		else
		{
			CP.Position = ReadCachePosition(CachePointIndex);
		}

		// Radius
		if (bHasRadiusData == 1)
		{
			CP.WorldRadius = ReadCacheRadius(CachePointIndex);
		}
		WriteHairControlPointPosition(OutDeformedBuffer, RestPointIndex, CP, DeformedPositionOffset, MaxHairRadius);
	}
}

#endif // SHADER_GROOMCACHE_UPDATE
