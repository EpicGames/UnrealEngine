// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../CommonViewUniformBuffer.ush"
#include "../SceneTextureParameters.ush"
#include "../DeferredShadingCommon.ush"
#include "HairStrandsCommon.ush"

#if SHADER_HAIRLUT 

#include "../ShadingModels.ush"

#ifdef HAIR_CUSTOM_BSDF
#include "../HairBsdf.ush"
#endif

#define PERMUTATION_LUT_TYPE_DUALSCATTERING 0
#define PERMUTATION_LUT_TYPE_MEAN_ENERGY 1

uint AbsorptionCount;
uint RoughnessCount;
uint ThetaCount;
uint SampleCountScale;
uint3 OutputResolution;
RWTexture3D<float4>	Out3D;
RWTexture2D<float4>	Out2D;

 float radicalInverse_VdC(uint bits) {
	bits = (bits << 16u) | (bits >> 16u);
	bits = ((bits & 0x55555555u) << 1u) | ((bits & 0xAAAAAAAAu) >> 1u);
	bits = ((bits & 0x33333333u) << 2u) | ((bits & 0xCCCCCCCCu) >> 2u);
	bits = ((bits & 0x0F0F0F0Fu) << 4u) | ((bits & 0xF0F0F0F0u) >> 4u);
	bits = ((bits & 0x00FF00FFu) << 8u) | ((bits & 0xFF00FF00u) >> 8u);
	return float(bits) * 2.3283064365386963e-10; // / 0x100000000
 }

float2 hammersley2d(uint i, uint N) 
{
	return float2(float(i)/float(N), radicalInverse_VdC(i));
}

#define TILE_PIXEL_SIZE 8
#define JITTER_VIEW 0

uint2 GetAtlasCoord2d(uint3 PixelCoord)
{
	const uint AbsorptionCountSqrt = uint(floor(sqrt(AbsorptionCount)));
	const uint2 TileCoord = uint2(PixelCoord.z % AbsorptionCountSqrt, PixelCoord.z / AbsorptionCountSqrt);
	return uint2(PixelCoord.x + TileCoord.x * ThetaCount, PixelCoord.y + TileCoord.y * RoughnessCount);
}

#if PERMUTATION_LUT_TYPE == PERMUTATION_LUT_TYPE_DUALSCATTERING

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, TILE_PIXEL_SIZE)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{	
// Plotting code of the BSDF
#if 0
	const uint2 PixelCoord = DispatchThreadId.xy;
	const float U = (float(PixelCoord.x % 64)+0.5f) / 64.0f;
	const float V = (PixelCoord.y+0.5f)/64.f; //saturate(float(PixelCoord.x / 32) / (32-1));
	const float W = (PixelCoord.x / 64) / 16.f;
	const float2 UV = float2(U,V);

	const float SinAngle	= 0.95; //0.95; //saturate(float(PixelCoord.y / ThetaCount));
	const float CosAngle 	= sqrt(1-SinAngle*SinAngle);
	const float3 View		= float3(CosAngle, 0, SinAngle); // float3(1, 0, 0)
	const float Absorption	= 1; //(PixelCoord.x + 64.5f) / 16.f;

	const float Roughness	= W;
	//const float Roughness = (PixelCoord.x+64.5f)/64.f * 0.1f;
	const float3 SampleDirection = UniformSampleSphere(UV).xyz;
	Out3D[PixelCoord] = float4(PlotHairBSDF(Roughness, SampleDirection, View, Absorption).xxx,1); // * (SampleDirection+1) * 0.5f
#endif


	// 3D LUT is organized as follow 
	//
	//      Z
	//	   ^
	//    /
	//   Absorption
	//  /
	// /
	//  ----- Theta ----> X
	// |
	// |
	// Roughness 
	// |
	// |
	// V
	// Y
	const uint3 PixelCoord = DispatchThreadId.xyz;
	const uint2 AtlasCoord = GetAtlasCoord2d(PixelCoord);

	const float SinAngle   	= saturate(float(PixelCoord.x+0.5f) / ThetaCount);
	const float Roughness  	= saturate(float(PixelCoord.y+0.5f) / RoughnessCount);
	const float Absorption 	= saturate(float(PixelCoord.z+0.5f) / AbsorptionCount);
	const float CosAngle 	= sqrt(1-SinAngle*SinAngle);

	FGBufferData GBufferData;
	GBufferData.Specular  	= 0.5f;
	GBufferData.BaseColor	= ToLinearAbsorption(Absorption.xxx);	// Perceptual absorption
	GBufferData.Metallic	= 0;		 							// This disable the fake multiple scattering
	GBufferData.Roughness 	= Roughness; 							// Perceptual roughness
	GBufferData.CustomData  = float4(0, 0, 1, 0); 					// Backlit

	float FrontHemisphereOutput = 0;
	float BackHemisphereOutput  = 0;

	uint FrontHemisphereCount = 0;
	uint BackHemisphereCount  = 0;
	
	const uint LocalThetaSampleCount	= max(1u, SampleCountScale * lerp(128, 64, Roughness));
	const uint LocalPhiSampleCount		= max(1u, SampleCountScale * lerp(128, 32, Roughness));
	const uint LocalViewSampleCount		= max(1u, SampleCountScale * 16);

	const float Area = 0;			// This is used for faking area light sources by increasing the roughness of the surface. Disabled = 0.
	const float Backlit = 1; 		// This is used for suppressing the R & TT terms when when the lighting direction comes from behind. Disabled = 1.
	const float3 N = float3(0,0,1); // N is the vector parallel to hair pointing toward root. I.e., the tangent T is up
	const float3 V = float3(CosAngle, 0, SinAngle);
	const float OpaqueVisibility = 1;
	FHairTransmittanceData TransmittanceData = InitHairStrandsTransmittanceData();
	TransmittanceData.bUseSeparableR = true;
	TransmittanceData.bClampBSDFValue = false;

	const float MaxCosThetaRadius = cos(0.25f * PI / float(ThetaCount)); // [0, Pi/2] / ThetaCount which is divided by 2 for getting the actual radius
	float3x3 ToViewBasis = GetTangentBasis(V);

	#if JITTER_VIEW == 1
	for (uint ViewIt=0; ViewIt<LocalViewSampleCount; ++ViewIt)
	#endif
	for (uint SampleItY=0; SampleItY<LocalPhiSampleCount; ++SampleItY)
	for (uint SampleItX=0; SampleItX<LocalThetaSampleCount; ++SampleItX)
	{	
		// Sample a small solid around the view direction in order to average the small differences
		// This allows to fight undersampling for low roughnesses
		#if JITTER_VIEW == 1
		const float2 ViewU = Hammersley(ViewIt, LocalViewSampleCount, 0);
		const float4 ViewSample = UniformSampleCone(ViewU, MaxCosThetaRadius);
		const float3 JitteredV = mul(ViewSample, ToViewBasis);
		const float ViewPdf = 1;
		#else
		const float3 JitteredV = V;
		const float ViewPdf = 1;
		#endif

		// Naive uniform sampling
		// @todo: important sampling of the Hair BSDF. The integration is too noisy for low roughness with uniform sampling
		const float2 jitter = R2Sequence(SampleItX + SampleItY * LocalThetaSampleCount); // float2(0.5f, 0.5f);
		const float2 u = (float2(SampleItX, SampleItY) + jitter) / float2(LocalThetaSampleCount, LocalPhiSampleCount);
		const float4 SampleDirection = UniformSampleSphere(u.yx);
		const float  SamplePdf = SampleDirection.w;
		const float3 L = SampleDirection.xyz;
        const float3 BSDFValue = HairShading(GBufferData, L, JitteredV, N, OpaqueVisibility, TransmittanceData, Backlit, Area, 0);
		
		// As in the original paper "Dual scattering approximation for fast multiple-scattering in hair", the average front/back scatter are cos-weighted (eq. 12). 
		const float CosL = 1.f;// abs(SampleDirection.x);

		// The view direction is aligned with the positive X Axis. This means:
		// * the back hemisphere (R / TRT) is on the positive side of X
		// * the front hemisphere (TT) is on the negative side of X
		const bool bIsBackHemisphere = SampleDirection.x > 0;
		if (bIsBackHemisphere)
		{
			BackHemisphereOutput += CosL * BSDFValue.x / SamplePdf;
			++BackHemisphereCount;
		}
		else
		{
			FrontHemisphereOutput += CosL * BSDFValue.x / SamplePdf;
			++FrontHemisphereCount;
		}
	}

	const float HemisphereFactor = 0.5f;
	const float4 OutValue = float4(
		saturate(FrontHemisphereOutput / FrontHemisphereCount * HemisphereFactor), 
		saturate(BackHemisphereOutput / BackHemisphereCount * HemisphereFactor),
		0, 1);

	Out3D[PixelCoord] = OutValue;
	Out2D[AtlasCoord] = OutValue;
}

#endif




#if PERMUTATION_LUT_TYPE == PERMUTATION_LUT_TYPE_MEAN_ENERGY

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, TILE_PIXEL_SIZE)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// 3D LUT is organized as follow 
	//
	//      Z
	//	   ^
	//    /
	//   Absorption
	//  /
	// /
	//  ----- Theta ----> X
	// |
	// |
	// Roughness 
	// |
	// |
	// V
	// Y
	const uint3 PixelCoord = DispatchThreadId.xyz;
	const uint2 AtlasCoord = GetAtlasCoord2d(PixelCoord);

	const float SinAngle = saturate(float(PixelCoord.x + 0.5f) / ThetaCount);
	const float Roughness = saturate(float(PixelCoord.y + 0.5f) / RoughnessCount);
	const float Absorption = saturate(float(PixelCoord.z + 0.5f) / AbsorptionCount);
	const float CosAngle = sqrt(1 - SinAngle * SinAngle);

	FGBufferData GBufferData;
	GBufferData.Specular = 0.5f;
	GBufferData.BaseColor = ToLinearAbsorption(Absorption.xxx);	// Perceptual absorption
	GBufferData.Metallic = 0;		 							// This disable the fake multiple scattering
	GBufferData.Roughness = Roughness; 							// Perceptual roughness
	GBufferData.CustomData = float4(0,0,1,0); 					// Backlit

	float R_Output = 0;
	float TT_Output = 0;
	float TRT_Output = 0;
	uint SampleCount = 0;

	const uint LocalThetaSampleCount = SampleCountScale * lerp(128, 64, Roughness);
	const uint LocalPhiSampleCount = SampleCountScale * lerp(128, 32, Roughness);
	const uint LocalViewSampleCount = SampleCountScale * 16;

	const float Area = 0;			// This is used for faking area light sources by increasing the roughness of the surface. Disabled = 0.
	const float Backlit = 1; 		// This is used for suppressing the R & TT terms when when the lighting direction comes from behind. Disabled = 1.
	const float3 N = float3(0, 0, 1); // N is the vector parallel to hair pointing toward root. I.e., the tangent T is up
	const float3 V = float3(CosAngle, 0, SinAngle);
	const float OpaqueVisibility = 1;
	FHairTransmittanceData Setting_R   = InitHairStrandsTransmittanceData(); Setting_R.ScatteringComponent		= HAIR_COMPONENT_R;
	FHairTransmittanceData Setting_TT  = InitHairStrandsTransmittanceData(); Setting_TT.ScatteringComponent		= HAIR_COMPONENT_TT;
	FHairTransmittanceData Setting_TRT = InitHairStrandsTransmittanceData(); Setting_TRT.ScatteringComponent	= HAIR_COMPONENT_TRT;

	const float MaxCosThetaRadius = cos(0.25f * PI / float(ThetaCount)); // [0, Pi/2] / ThetaCount which is divided by 2 for getting the actual radius
	float3x3 ToViewBasis = GetTangentBasis(V);

	#if JITTER_VIEW == 1
	for (uint ViewIt = 0; ViewIt < LocalViewSampleCount; ++ViewIt)
	#endif
	for (uint SampleItY = 0; SampleItY < LocalPhiSampleCount; ++SampleItY)
	for (uint SampleItX = 0; SampleItX < LocalThetaSampleCount; ++SampleItX)
	{
		// Sample a small solid around the view direction in order to average the small differences
		// This allows to fight undersampling for low roughnesses
		#if JITTER_VIEW == 1
		const float2 ViewU = Hammersley(ViewIt, LocalViewSampleCount, 0);
		const float4 ViewSample = UniformSampleCone(ViewU, MaxCosThetaRadius);
		const float3 JitteredV = mul(ViewSample, ToViewBasis);
		const float ViewPdf = 1;
		#else
		const float3 JitteredV = V;
		const float ViewPdf = 1;
		#endif

		// Naive uniform sampling
		// @todo: important sampling of the Hair BSDF. The integration is too noisy for low roughness with uniform sampling
		const float2 jitter = R2Sequence(SampleItX + SampleItY * LocalThetaSampleCount); // float2(0.5f, 0.5f);
		const float2 u = (float2(SampleItX, SampleItY) + jitter) / float2(LocalThetaSampleCount, LocalPhiSampleCount);
		const float4 SampleDirection = UniformSampleSphere(u.yx);
		const float  SamplePdf = SampleDirection.w;
		const float3 L = SampleDirection.xyz;
		const float3 BSDFValue_R   = HairShading(GBufferData, L, JitteredV, N, OpaqueVisibility, Setting_R, Backlit, Area, 0);
		const float3 BSDFValue_TT  = HairShading(GBufferData, L, JitteredV, N, OpaqueVisibility, Setting_TT, Backlit, Area, 0);
		const float3 BSDFValue_TRT = HairShading(GBufferData, L, JitteredV, N, OpaqueVisibility, Setting_TRT, Backlit, Area, 0);

		R_Output   += BSDFValue_R.x   / SamplePdf;
		TT_Output  += BSDFValue_TT.x  / SamplePdf;
		TRT_Output += BSDFValue_TRT.x / SamplePdf;
		++SampleCount;
	}

	const float4 OutValue = float4(
		saturate(R_Output / SampleCount),
		saturate(TT_Output / SampleCount),
		saturate(TRT_Output / SampleCount),
		1);

	Out3D[PixelCoord] = OutValue;
	Out2D[AtlasCoord] = OutValue;
}

#endif

#endif // SHADER_HAIRLUT


#if SHADER_HAIRCOVERAGE

int2 OutputResolution;
Buffer<float> InputBuffer;
RWTexture2D<float> OutputTexture;

#define TILE_PIXEL_SIZE 8

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const int2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord > OutputResolution))
	{
		return;
	}

	const uint Index = PixelCoord.x + PixelCoord.y * OutputResolution.x;
	OutputTexture[PixelCoord] = InputBuffer[Index];
}
#endif // SHADER_HAIRCOVERAGE
