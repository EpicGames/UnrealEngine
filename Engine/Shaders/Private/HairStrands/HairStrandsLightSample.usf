// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"

///////////////////////////////////////////////////////////////////////////

#if SHADER_VERTEX

int2 MaxViewportResolution;
StructuredBuffer<uint> HairNodeCountBuffer;

void MainVS(
	in uint InVertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	nointerpolation out uint OutNodeCount : DISPATCH_NODECOUNT,
	nointerpolation out uint2 OutResolution : DISPATCH_RESOLUTION)
{
	OutNodeCount = HairNodeCountBuffer[0];
	OutResolution = GetHairSampleResolution(OutNodeCount);

	const float2 ClipCoord = ((OutResolution + 0.5f) / float2(MaxViewportResolution)) * 2;

	const float2 UV = float2(InVertexId & 1, InVertexId >> 1);
	const float2 Pos = float2(-UV.x, UV.y) * 4 + float2(-1, +1) + float2(ClipCoord.x, -ClipCoord.y);
	OutPosition = float4(Pos, 0.5f, 1);
}

#endif


#if SHADER_CLEAR

void ClearPS(
	float4 SVPos : SV_POSITION,
	uint NodeCount : DISPATCH_NODECOUNT,
	uint2 Resolution : DISPATCH_RESOLUTION,
	out float4 OutColor : SV_Target0)
{
	const uint2 PixelCoord = uint2(SVPos.xy);
	if (any(PixelCoord > Resolution))
		discard;

	OutColor = 0;
}
#endif