// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
float3 DecodeVelocityFromTexture(float4 In);
#include "HairStrandsVisibilityCommon.ush" 

// Note: Culled cluster is disabled as the input to the vertex factory from HairStrandsMaterialPS or HairStrandsMaterialGBufferPS 
// are alredy culled vertex (coming from the visibility buffer). By disabling the cluster within this material pass, we insure that 
// the provided primitive ID will be used for feteching attrinbute, rather than the one for conmputing the indirect ID 
// 
// This is required as the hair strands vertex factory enabled culled cluster by default (to support shadow & hit proxy shader for 
// which we can't provide custom defines)
#define USE_CULLED_CLUSTER 0

#define USE_FORCE_TEXTURE_MIP 1
#define FORCED_TEXTURE_MIP 0.0f
#include "../Common.ush"

#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "../VelocityCommon.ush"

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush" 

#include "HairStrandsVertexFactoryCommon.ush"

#ifndef USE_HAIR_TRIANGLE_STRIP
	#error Hair triangle geometry type needs to be defined
#endif

#define SUPPORT_MATERIAL_PROPERTY 1
uint MaterialPass_MacroGroupId;
uint MaterialPass_MaterialId;  
uint MaterialPass_PrimitiveId;
uint MaterialPass_LightChannelMask;
uint MaterialPass_Flags;
float MaterialPass_HairCoverageScale;

#ifndef HAIR_MATERIAL_DEBUG_OUTPUT
#define HAIR_MATERIAL_DEBUG_OUTPUT 0
#endif

#if HAIR_STRAND_MESH_FACTORY
struct FEvaluateOutput
{
	FHairSample NodeData;
	float4 NodeVelocity;
};

struct FHairVertexData
{
	float3 World_P;
	float3 World_P_Prev;
	float  Radius;
	bool   bIsValid;
};

// GPUCULL_TODO: this should be switched over to use Instance ID instead of Primitive ID.
FVertexFactoryInterpolantsVSToPS GetInterpolants(uint PrimitiveId, uint HairControlPointId, inout FHairVertexData OutHairVertex)
{
	FMaterialVertexParameters Empty = MakeInitializedMaterialVertexParameters();

	FVertexFactoryInput Input;
	// Assumes no actual instancing used for these draws. Will load the first instance data.
	VF_GPUSCENE_SET_INPUT_FOR_HAIR(Input, PrimitiveId, 0U);
	// * USE_HAIR_TRIANGLE_STRIP=0: the vertex factory expect triangles strip index. A quad is made of two triangles with indices: 0,1.
	// * USE_HAIR_TRIANGLE_STRIP=1: the vertex factory expect triangles list index. A quad is made of two triangles with indices: 0,1,2,3,4,5.
	Input.VertexId = HairControlPointId * HAIR_POINT_TO_VERTEX; 

	FVertexFactoryIntermediates	Intermediates = GetVertexFactoryIntermediates(Input);
	FVertexFactoryInterpolantsVSToPS Interpolants = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, Empty);

	OutHairVertex.World_P = VertexFactoryGetWorldPositionRaw(Input, Intermediates, false).xyz;
	OutHairVertex.World_P_Prev = VertexFactoryGetPreviousWorldPosition(Input, Intermediates, false).xyz;

	const FHairControlPoint ControlPoint = GetVertexPosition(Input, false);
	OutHairVertex.Radius = ControlPoint.WorldRadius;
	OutHairVertex.bIsValid = ControlPoint.Type != HAIR_CONTROLPOINT_END;

	return Interpolants;
}

FEvaluateOutput Evaluate(
	in float2 SamplePixelCoord,
	in float SampleDepth,
	in uint SampleControlPointId,
	in uint SampleCoverage8bit,
	in uint PrimitiveId,
	in bool bUpdateSampleCoverage,
	in bool bInterpolationEnabled)
{
	// World_P / World_P0 / World_P1 are in translated world space
	const float SceneDepth = ConvertFromDeviceZ(SampleDepth);
	const float2 UV = SamplePixelCoord / float2(View.BufferSizeAndInvSize.xy);
	const float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	const float3 World_P = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;

	float3 Velocity_World_P;
	float3 Velocity_World_P_Prev;
	float WorldStrandRadius = 0;
	FVertexFactoryInterpolantsVSToPS Interpolants;
	{
		const uint HairControlPointId0 = SampleControlPointId;

		FHairVertexData Vertex0;
		FVertexFactoryInterpolantsVSToPS Interpolants0 = GetInterpolants(PrimitiveId, HairControlPointId0, Vertex0);
		Interpolants = Interpolants0;

		float SegmentU = 0.0f;
		float SegmentV = 0.5f;
		if (bInterpolationEnabled)
		{
			const uint HairControlPointId1 = HairControlPointId0 + 1; //TODO: min(HairControlPointId0 + 1, MaxVertexCount-1);

			FHairVertexData Vertex1;
			FVertexFactoryInterpolantsVSToPS Interpolants1 = GetInterpolants(PrimitiveId, HairControlPointId1, Vertex1);

			// Compute U
			// Use the projection of the current sample point onto the hair segment (which is in the middle of the hair strands)
			const float3 A = (World_P  - Vertex0.World_P);
			const float3 B = (Vertex1.World_P - Vertex0.World_P);
			{
				const float CosA = dot(A, B);
				const float LengthB2 = dot(B, B);
				SegmentU = LengthB2 > 0 ? saturate(CosA / LengthB2) : 0;
			}

			// Compute V
			{
				const float3 W = cross(A, B);
				const bool bIsRight = dot(W, View.ViewForward) < 0;
				const float WorldV	= length(A - B * SegmentU);
				WorldStrandRadius	= lerp(Vertex0.Radius, Vertex1.Radius, SegmentU);
				SegmentV = WorldV / WorldStrandRadius;
				SegmentV = bIsRight ? SegmentV * 0.5f + 0.5f : (1 - SegmentV) * 0.5f;
			}

			Velocity_World_P	  = lerp(Vertex0.World_P, Vertex1.World_P, SegmentU);
			Velocity_World_P_Prev = lerp(Vertex0.World_P_Prev, Vertex1.World_P_Prev, SegmentU);

			Interpolants.TangentToWorld0 = lerp(Interpolants0.TangentToWorld0, Interpolants1.TangentToWorld0, SegmentU);
			Interpolants.TangentToWorld2 = lerp(Interpolants0.TangentToWorld2, Interpolants1.TangentToWorld2, SegmentU);
		}
		else
		{
			WorldStrandRadius				= Vertex0.Radius;
			Interpolants.TangentToWorld0	= Interpolants0.TangentToWorld0;
			Interpolants.TangentToWorld2	= Interpolants0.TangentToWorld2;
			Velocity_World_P				= Vertex0.World_P;
			Velocity_World_P_Prev			= Vertex0.World_P_Prev;
		}

		Interpolants.TangentToWorld0 = normalize(Interpolants.TangentToWorld0);
		Interpolants.TangentToWorld2 = normalize(Interpolants.TangentToWorld2);

#if VF_USE_PRIMITIVE_SCENE_DATA
		Interpolants.InstanceId	 = GetPrimitiveData(PrimitiveId).InstanceSceneDataOffset;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
		Interpolants.HairControlPointId = HairControlPointId0;
		Interpolants.HairPrimitiveUV = float2(SegmentU, SegmentV);
	}
	
	float4 EncodedVelocity = 0;
	{
		const float4 ScreenPos		= mul(float4(Velocity_World_P.xyz, 1), ResolvedView.TranslatedWorldToClip);
		const float4 PrevScreenPos	= mul(float4(Velocity_World_P_Prev.xyz, 1), ResolvedView.PrevTranslatedWorldToClip);

		const float3 Velocity	= Calculate3DVelocity(ScreenPos, PrevScreenPos);
		EncodedVelocity			= EncodeVelocityToTexture(Velocity);
	}

	// Sample Position < consider to be the center??
	// VS
	const float4 SvPosition = float4(SamplePixelCoord, SampleDepth, 1);

	// Coverage computation
	// We don't use the coverage information
	float Coverage = 1;
	if (bUpdateSampleCoverage)
	{
		const bool bUseStableRasterization = UseStableRasterization();

		FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits, bUseStableRasterization);
		const float LocalSceneDepth = ConvertFromDeviceZ(SvPosition.z); // Linear depth in world unit
		const float PixelRadius = ConvertGivenDepthRadiusForProjectionType(HairRenderInfo.RadiusAtDepth1Primary, LocalSceneDepth); // Not correct but the coverage is not used (we count instead the number of sub-sample covered)
		Coverage = saturate(WorldStrandRadius / max(WorldStrandRadius, PixelRadius) * MaterialPass_HairCoverageScale);

		SampleCoverage8bit = min(uint(Coverage * 0xFF), 0xFFu);
	}

	// expressed the coverage relatively to the current pixel coverage?

	// PS
	FHairSample OutSample		= (FHairSample)0;
	OutSample.Depth				= SampleDepth;
	OutSample.Coverage8bit		= SampleCoverage8bit;
	OutSample.ControlPointId	= SampleControlPointId;
	OutSample.MacroGroupId		= MaterialPass_MacroGroupId;
	OutSample.LightChannelMask	= MaterialPass_LightChannelMask;
	OutSample.Flags 			= MaterialPass_Flags;

	// Material computation
	{
		const bool bIsFrontFace = true;
		FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
		FPixelMaterialInputs PixelMaterialInputs;

		float4 LocalScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, LocalScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);

#if MATERIAL_IS_SUBSTRATE && MATERIAL_SHADINGMODEL_HAIR
		FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
		SubstratePixelHeader.SetCastContactShadow(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS);
		SubstratePixelHeader.SetDynamicIndirectShadowCasterRepresentation(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION);

		float3 Tangent = float3(0,1,0);
		//if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
		{
			// Using the tree data, instread of the inline BSDF does not seem to work. It is unclear why. It is not very 
			// important as we expect a single hair BSDF.
			// Update tree (coverage/transmittance/luminace weights)
			const float3 V = MaterialParameters.CameraVector;
			const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuse*/, -1 /*PeelLayersAboveDepth*/, false/*bRoughnessTracking*/);
			SubstratePixelHeader.SubstrateUpdateTree(V, Settings);
			FSubstrateBSDF HairBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];

			OutSample.BaseColor = HAIR_BASECOLOR(HairBSDF);
			OutSample.Roughness = HAIR_ROUGHNESS(HairBSDF);
			OutSample.Specular	= HAIR_SPECULAR(HairBSDF);
			OutSample.Backlit	= HAIR_BACKLIT(HairBSDF);
			OutSample.Emissive	= BSDF_GETEMISSIVE(HairBSDF);
			Tangent				= normalize(MaterialParameters.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(HairBSDF)]);
		}
		
	#if HAIRSTRANDS_HAS_NORMAL_CONNECTED
	  #if MATERIAL_TANGENTSPACENORMAL 
		// Tangent is in world space. First transform it into local space prior rotation)
		float3 LocalTangent = mul(Tangent, transpose(MaterialParameters.TangentToWorld));
		OutSample.Tangent = normalize(mul(RotateLocalHairTangent(LocalTangent), MaterialParameters.TangentToWorld));
	  #else // Tangent in World space (likely provided by the HairAttributes node)
		OutSample.Tangent = Tangent;
	  #endif
	#else
		OutSample.Tangent = normalize(Interpolants.TangentToWorld2.xyz);
	#endif
#else

		OutSample.BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
		OutSample.Roughness = GetMaterialRoughness(PixelMaterialInputs);
		OutSample.Specular	= GetMaterialSpecular(PixelMaterialInputs);
		OutSample.Backlit	= saturate(GetMaterialCustomData0(PixelMaterialInputs));
		OutSample.Emissive	= GetMaterialEmissive(PixelMaterialInputs);
		float3 Tangent		= normalize(GetMaterialNormalRaw(PixelMaterialInputs));

	#if HAIRSTRANDS_HAS_NORMAL_CONNECTED
	  #if MATERIAL_TANGENTSPACENORMAL 
		OutSample.Tangent = normalize(mul(RotateLocalHairTangent(Tangent), MaterialParameters.TangentToWorld));
	  #else // Tangent in World space (likely provided by the HairAttributes node)
		OutSample.Tangent = Tangent;
	  #endif
	#else
		OutSample.Tangent = normalize(Interpolants.TangentToWorld2.xyz);
	#endif
#endif

		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	#if USE_DEVELOPMENT_SHADERS
		{
			OutSample.BaseColor = OutSample.BaseColor * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
		}
	#endif
	}

	FEvaluateOutput Out;
	Out.NodeData	 = OutSample;
	Out.NodeVelocity = EncodedVelocity;
	return Out;
}
#endif
