// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsMaterialCommon.ush"

void Main(
	in float4 SvPosition: SV_Position
	, in uint2 Resolution : DISPATCH_RESOLUTION
	, in uint TotalNodeCount : DISPATCH_NODE_COUNT
	, in uint InPrimitiveId : PRIMITIVE_ID
	#if HAIR_MATERIAL_EMISSIVE_OUTPUT
	, out float4 OutColor0 : SV_Target0
	#endif
	)
{
	ResolvedView = ResolveView();

	const uint3 PixelCoord = uint3(floor(SvPosition.xy), 0);
	const uint SampleIndex = PixelCoord.x + PixelCoord.y * Resolution.x;// MaterialPassParameters.MaxResolution.x;
	bool bIsValid = false;
	float3 Emissive = 0;
	if (SampleIndex < TotalNodeCount)
	{
		const FHairVis InNode = UnpackHairVis(MaterialPassParameters.NodeVis[SampleIndex]);
		const float2 SamplePixelCoord = float2(MaterialPassParameters.NodeCoord[SampleIndex]) + float2(0.5f, 0.5f);

		float SampleDepth = InNode.Depth;
		uint SampleCoverage8bit = InNode.Coverage8bit;
		uint SampleControlPointId = InNode.ControlPointId;
		uint SampleMaterialId = InNode.MaterialId;

		if (SampleMaterialId == MaterialPass_MaterialId)
		{
			FEvaluateOutput Out = (FEvaluateOutput)0;
			if (SampleDepth > 0)
			{
				Out = Evaluate(SamplePixelCoord, SampleDepth, SampleControlPointId, SampleCoverage8bit, InPrimitiveId, MaterialPassParameters.bUpdateSampleCoverage > 0, MaterialPassParameters.bInterpolationEnabled);
			}

			// Note: Use the syntax _ over the syntax . as on certain platform the UAV buffers are 
			// stored within a static const struct preventing doing a write operation
			//
			// Original code:
			// MaterialPassParameters.OutNodeData[SampleIndex] = PackHairSample(Out.NodeData);
			// MaterialPassParameters.OutNodeVelocity[SampleIndex] = Out.NodeVelocity;
			MaterialPassParameters_OutNodeData[SampleIndex] = PackHairSample(Out.NodeData);
			MaterialPassParameters_OutNodeVelocity[SampleIndex] = Out.NodeVelocity;

			bIsValid = true;
			Emissive = Out.NodeData.Emissive * float(Out.NodeData.Coverage8bit / 255.f);
		}
	}

	#if HAIR_MATERIAL_EMISSIVE_OUTPUT
	OutColor0 = float4(Emissive, 1);
	#endif
}


