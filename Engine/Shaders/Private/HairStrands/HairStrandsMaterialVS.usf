// Copyright Epic Games, Inc. All Rights Reserved.

float3 DecodeVelocityFromTexture(float4 In);
#include "HairStrandsVisibilityCommon.ush" 
#include "../Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#if VERTEXSHADER
 
void Main(
	FVertexFactoryInput Input,
	out float4 Position : SV_POSITION,
	nointerpolation out uint2 OutResolution : DISPATCH_RESOLUTION,
	nointerpolation out uint  OutTotalNodeCount : DISPATCH_NODE_COUNT,
	nointerpolation out uint  OutPrimitiveId : PRIMITIVE_ID)
{
	OutTotalNodeCount = MaterialPassParameters.TotalNodeCounter[0];
	OutResolution = GetHairSampleResolution(OutTotalNodeCount);

	const float2 ClipCoord = ((OutResolution +0.5f) / float2(MaterialPassParameters.MaxResolution)) * 2;

	const float2 UV = float2(Input.VertexId & 1, Input.VertexId >> 1);
	const float2 Pos = float2(-UV.x, UV.y) * 4 + float2(-1,+1) + float2(ClipCoord.x, -ClipCoord.y);
	Position = float4(Pos, 0.5f, 1);

	const FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	OutPrimitiveId = VFIntermediates.SceneData.PrimitiveId; 
}

#endif // VERTEXSHADER
