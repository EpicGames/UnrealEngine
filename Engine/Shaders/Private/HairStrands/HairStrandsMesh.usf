// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HairStrandsBindingCommon.ush"
#include "HairstrandsVertexFactoryCommon.ush"

////////////////////////////////////////////////////////////////////////
// Deform hair mesh basd on RBF weights

#if SHADER_HAIRMESHES

uint VertexCount;
uint MaxSampleCount;
uint RBFLocalSpace;

StructuredBuffer<float4> RestSamplePositionsBuffer;
StructuredBuffer<float4> MeshSampleWeightsBuffer;
StructuredBuffer<float4> DeformedSamplePositionsBuffer;

Buffer<float4> RestPositionBuffer;
RWBuffer<float4> OutDeformedPositionBuffer;

[numthreads(128, 1, 1)]
void MainHairMeshesCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint VertexIndex = DispatchThreadId.x;
	if (VertexIndex >= VertexCount)
		return;

	const float3 RestPosition = RestPositionBuffer[VertexIndex].xyz;
	const float3 DeformedPosition = ApplyRBF(RestPosition, MaxSampleCount, RestSamplePositionsBuffer, MeshSampleWeightsBuffer, DeformedSamplePositionsBuffer, RBFLocalSpace);
	OutDeformedPositionBuffer[VertexIndex] = float4(DeformedPosition,1);
}

#endif // SHADER_HAIRMESHES

////////////////////////////////////////////////////////////////////////
// Init RBF point based on mesh position

#if SHADER_SAMPLE_INIT

#if PERMUTATION_POSITION_TYPE == 0
Buffer<float> MeshPositionBuffer0;
Buffer<float> MeshPositionBuffer1;
#elif PERMUTATION_POSITION_TYPE == 1
Buffer<float> RDGMeshPositionBuffer0;
Buffer<float> RDGMeshPositionBuffer1;
#define MeshPositionBuffer0 RDGMeshPositionBuffer0
#define MeshPositionBuffer1 RDGMeshPositionBuffer1
#endif

uint MaxSectionCount;
uint MaxVertexCount;
uint MaxSampleCount;
Buffer<uint> SampleIndicesAndSectionsBuffer;
StructuredBuffer<uint> MeshSectionBuffer;
RWStructuredBuffer<float4> OutSamplePositionsBuffer;

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint SampleIndex = DispatchThreadId.x;
	if (SampleIndex >= MaxSampleCount) 
		return;

	const FHairTriangleIndex IndexAndSection = UnpackTriangleIndex(SampleIndicesAndSectionsBuffer[SampleIndex]);
	const uint VertexIndex = IndexAndSection.TriangleIndex;
	const uint SectionIndex = IndexAndSection.SectionIndex;
	if (VertexIndex >= MaxVertexCount || SectionIndex >= MaxSectionCount) 
		return;
	
	const bool bBuffer0 = MeshSectionBuffer[SectionIndex] == 0;

	float3 SamplePosition = float3(0,0,0);
	if (bBuffer0)
	{
		SamplePosition.x = MeshPositionBuffer0.Load(VertexIndex * 3 + 0);
		SamplePosition.y = MeshPositionBuffer0.Load(VertexIndex * 3 + 1);
		SamplePosition.z = MeshPositionBuffer0.Load(VertexIndex * 3 + 2);
	}
	else
	{
		SamplePosition.x = MeshPositionBuffer1.Load(VertexIndex * 3 + 0);
		SamplePosition.y = MeshPositionBuffer1.Load(VertexIndex * 3 + 1);
		SamplePosition.z = MeshPositionBuffer1.Load(VertexIndex * 3 + 2);
	}
	OutSamplePositionsBuffer[SampleIndex] = float4(SamplePosition,1);
}

#endif // SHADER_SAMPLE_INIT

////////////////////////////////////////////////////////////////////////
// Update RBF points

#if SHADER_SAMPLE_UPDATE

// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

uint MaxSampleCount;
uint RBFLocalSpace;
StructuredBuffer<float> InterpolationWeightsBuffer;

StructuredBuffer<float4> SampleRestPositionsBuffer;
StructuredBuffer<float4> SampleDeformedPositionsBuffer;

RWStructuredBuffer<float4> OutSampleDeformationsBuffer; 

[numthreads(128, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{ 
	const uint SampleIndex = DispatchThreadId.x;
	if (SampleIndex >= HAIR_RBF_ENTRY_COUNT(MaxSampleCount))
	{
		return;
	}

	const bool bEnableLocalSpace = (MaxSampleCount > 0) && (RBFLocalSpace == 1);
	const float4 RestSampleOffset = bEnableLocalSpace ? SampleRestPositionsBuffer[0] : float4(0,0,0,0);
	const float4 DeformedSampleOffset = bEnableLocalSpace ? SampleDeformedPositionsBuffer[0] : float4(0,0,0,0);

	uint WeightsBaseOffset = SampleIndex * HAIR_RBF_ENTRY_COUNT(MaxSampleCount);
	float3 SampleDeformation = float3(0,0,0);
	for(uint i = 0; i < MaxSampleCount; ++i)
	{
		//@todo-laura.hermanns - Incrementing '++WeightsBaseOffset' instead of using primary iterator '+i' results in visual artifacts with Metal backend
		SampleDeformation += InterpolationWeightsBuffer[WeightsBaseOffset + i] * ((SampleDeformedPositionsBuffer[i]-DeformedSampleOffset) -
			(SampleRestPositionsBuffer[i]-RestSampleOffset)).xyz;
	}

	OutSampleDeformationsBuffer[SampleIndex] = float4(SampleDeformation,1);
}

#endif // SHADER_SAMPLE_UPDATE

////////////////////////////////////////////////////////////////////////
// Update strands root triangle positions

#if SHADER_MESH_UPDATE

#ifndef GROUP_SIZE
	#error GROUP_SIZE needs to be defined 
#endif

struct FMeshSectionData
{
	uint MaxIndexCount;
	uint MaxVertexCount;
	uint IndexOffset;
	uint UVsChannelOffset;
	uint UVsChannelCount;
	uint bIsSwapped;
	uint bUseFaceNormal;
};

uint TangentFormat;
uint MaxSectionCount;
uint MaxUniqueTriangleCount;
Buffer<uint> MeshIndexBuffer;
Buffer<uint> UniqueTriangleIndices;
StructuredBuffer<uint4> MeshSectionBuffer;

// Note on buffer input type
// There are two different input buffer permutation, as if skin cache is enabled, we are using raw RHI SRV 
// (while the skin cache system is not converted to RDG). If skin cache is disable, a manual skin compute 
// update happens in the hair system, and its output are RDG buffers.
#if PERMUTATION_POSITION_TYPE == 0
Buffer<float4> MeshTangentBuffer;
Buffer<float> MeshPositionBuffer;
Buffer<float> MeshPreviousPositionBuffer;
#else
Buffer<float4> RDGMeshTangentBuffer;
Buffer<float> RDGMeshPositionBuffer;
Buffer<float> RDGMeshPreviousPositionBuffer;
#define MeshTangentBuffer RDGMeshTangentBuffer
#define MeshPositionBuffer RDGMeshPositionBuffer
#define MeshPreviousPositionBuffer RDGMeshPreviousPositionBuffer
#endif 

RWBuffer<float4> OutUniqueTriangleCurrPosition;
#if PERMUTATION_PREVIOUS
RWBuffer<float4> OutUniqueTrianglePrevPosition;
#endif

FMeshSectionData ReadMeshSectionData(uint InSectionIndex)
{
	const uint4 Packed = MeshSectionBuffer[InSectionIndex];

	FMeshSectionData Out;
	Out.MaxIndexCount = Packed.x;
	Out.MaxVertexCount = Packed.y;
	Out.IndexOffset = Packed.z;
	Out.UVsChannelOffset = BitFieldExtractU32(Packed.w, 8, 0);
	Out.UVsChannelCount = BitFieldExtractU32(Packed.w, 8, 8);
	Out.bIsSwapped = BitFieldExtractU32(Packed.w, 1, 16);
	Out.bUseFaceNormal = BitFieldExtractU32(Packed.w, 1, 17);
	return Out;
}

FHairMeshTriangle GetHairMeshTriangle(uint InTriangleIndex, FMeshSectionData MeshSectionData, bool bPrevious, bool bUseVertexTangent)
{
	const bool bSamplePrev = (MeshSectionData.bIsSwapped > 0 && !bPrevious) || (MeshSectionData.bIsSwapped == 0 && bPrevious);
	if (bSamplePrev)
	{
		return GetHairMeshTriangle(
			InTriangleIndex, 
			MeshPreviousPositionBuffer, 
			MeshTangentBuffer,
			MeshIndexBuffer, 
			MeshSectionData.IndexOffset,
			MeshSectionData.MaxIndexCount,
			MeshSectionData.MaxVertexCount,
			TangentFormat,
			bUseVertexTangent);
	}
	else
	{
		return GetHairMeshTriangle(
			InTriangleIndex, 
			MeshPositionBuffer, 
			MeshTangentBuffer,
			MeshIndexBuffer, 
			MeshSectionData.IndexOffset,
			MeshSectionData.MaxIndexCount,
			MeshSectionData.MaxVertexCount,
			TangentFormat,
			bUseVertexTangent);
	}
}

float3 GetTriangleFaceNormal(FHairMeshTriangle Tri)
{
	const float3 E0 = Tri.P1 - Tri.P0;
	const float3 E1 = Tri.P2 - Tri.P0;
	return normalize(cross(E1, E0));
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{ 
	const uint UniqueTriangleIndex = DispatchThreadId.x;
	if (UniqueTriangleIndex >= MaxUniqueTriangleCount)
	{
		return;
	}
	
	const FHairTriangleIndex TriangleIndex = UnpackTriangleIndex(UniqueTriangleIndices[UniqueTriangleIndex]);
	if (TriangleIndex.SectionIndex < MaxSectionCount)
	{
		const FMeshSectionData MeshSectionData = ReadMeshSectionData(TriangleIndex.SectionIndex);
	
		// Compute the slot index of the resources for the section index (this is done in multiple pass
		{
			FHairMeshTriangle Tri = GetHairMeshTriangle(TriangleIndex.TriangleIndex, MeshSectionData, false, !MeshSectionData.bUseFaceNormal);
			if (MeshSectionData.bUseFaceNormal)
			{
				Tri.N0 = GetTriangleFaceNormal(Tri);
				Tri.N1 = Tri.N0;
				Tri.N2 = Tri.N0;
			}
			OutUniqueTriangleCurrPosition[UniqueTriangleIndex * 3 + 0] = float4(Tri.P0, PackVertexNormalToFloat(Tri.N0));
			OutUniqueTriangleCurrPosition[UniqueTriangleIndex * 3 + 1] = float4(Tri.P1, PackVertexNormalToFloat(Tri.N1));
			OutUniqueTriangleCurrPosition[UniqueTriangleIndex * 3 + 2] = float4(Tri.P2, PackVertexNormalToFloat(Tri.N2));
		}
		#if PERMUTATION_PREVIOUS
		{
			FHairMeshTriangle Tri = GetHairMeshTriangle(TriangleIndex.TriangleIndex, MeshSectionData, true, !MeshSectionData.bUseFaceNormal);
			if (MeshSectionData.bUseFaceNormal)
			{
				Tri.N0 = GetTriangleFaceNormal(Tri);
				Tri.N1 = Tri.N0;
				Tri.N2 = Tri.N0;
			}
			OutUniqueTrianglePrevPosition[UniqueTriangleIndex * 3 + 0] = float4(Tri.P0, PackVertexNormalToFloat(Tri.N0));
			OutUniqueTrianglePrevPosition[UniqueTriangleIndex * 3 + 1] = float4(Tri.P1, PackVertexNormalToFloat(Tri.N1));
			OutUniqueTrianglePrevPosition[UniqueTriangleIndex * 3 + 2] = float4(Tri.P2, PackVertexNormalToFloat(Tri.N2));
		}
		#endif
	}
}

#endif // SHADER_MESH_UPDATE

////////////////////////////////////////////////////////////////////////
// Update hair strands offset position

#if SHADER_OFFSET_UPDATE

#if PERMUTATION_USE_GPU_OFFSET
	Buffer<float4> RootTriangleCurrPositionBuffer;
	float3 GetOffsetCurrPosition()	{ return RootTriangleCurrPositionBuffer[0].xyz; }

	#if PERMUTATION_PREVIOUS
	Buffer<float4> RootTrianglePrevPositionBuffer;
	float3 GetOffsetPrevPosition()	{ return RootTrianglePrevPositionBuffer[0].xyz; }
	#endif
#else
	float3 CPUCurrPositionOffset;
	float3 CPUPrevPositionOffset;

	float3 GetOffsetCurrPosition()	{ return CPUCurrPositionOffset; }
	float3 GetOffsetPrevPosition()	{ return CPUPrevPositionOffset; }
#endif

RWStructuredBuffer<float4> OutCurrOffsetBuffer;
#if PERMUTATION_PREVIOUS
RWStructuredBuffer<float4> OutPrevOffsetBuffer;
#endif

uint UpdateType;
uint CardLODIndex;
uint InstanceRegisteredIndex;

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x > 0)
		return;

	WritePositionOffset(OutCurrOffsetBuffer, InstanceRegisteredIndex, UpdateType, HAIR_CURR, CardLODIndex, GetOffsetCurrPosition());
#if PERMUTATION_PREVIOUS
	WritePositionOffset(OutPrevOffsetBuffer, InstanceRegisteredIndex, UpdateType, HAIR_PREV, CardLODIndex, GetOffsetPrevPosition());
#endif
}

#endif // SHADER_OFFSET_UPDATE


////////////////////////////////////////////////////////////////////////
// Update hair strands offset position

#if SHADER_RESOURCE_TRANSITION

#define TEMPLATE_BUFFERS(TPrefix, TSuffix) \
	TPrefix##Buffers_0##TSuffix;\
	TPrefix##Buffers_1##TSuffix;\
	TPrefix##Buffers_2##TSuffix;\
	TPrefix##Buffers_3##TSuffix;\
	TPrefix##Buffers_4##TSuffix;\
	TPrefix##Buffers_5##TSuffix;\
	TPrefix##Buffers_6##TSuffix;\
	TPrefix##Buffers_7##TSuffix;\
	TPrefix##Buffers_8##TSuffix;\
	TPrefix##Buffers_9##TSuffix;\
	TPrefix##Buffers_10##TSuffix;\
	TPrefix##Buffers_11##TSuffix;\
	TPrefix##Buffers_12##TSuffix;\
	TPrefix##Buffers_13##TSuffix;\
	TPrefix##Buffers_14##TSuffix;\
	TPrefix##Buffers_15##TSuffix;

TEMPLATE_BUFFERS(StructuredBuffer<float4> Structured, )
TEMPLATE_BUFFERS(Buffer<uint> VertexUInt, )
TEMPLATE_BUFFERS(Buffer<uint4> VertexUInt4, )
TEMPLATE_BUFFERS(Buffer<float4> VertexFloat4, )
TEMPLATE_BUFFERS(ByteAddressBuffer ByteAddress, )

RWStructuredBuffer<uint> DummyOutput;
uint DummyValue;
uint NumVertexUint;
uint NumVertexUint4;
uint NumVertexFloat4;

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DummyValue > 0)
	{
		uint Acc = 0;
	#if PERMUTATION_BUFFER_TYPE == 0
		TEMPLATE_BUFFERS(Acc += Structured, [0].x)
	#elif PERMUTATION_BUFFER_TYPE == 1
		TEMPLATE_BUFFERS(Acc += VertexUInt, [0])
	#elif PERMUTATION_BUFFER_TYPE == 2
		TEMPLATE_BUFFERS(Acc += VertexUInt4, [0].x)
	#elif PERMUTATION_BUFFER_TYPE == 3
		TEMPLATE_BUFFERS(Acc += VertexFloat4, [0].x)
	#elif PERMUTATION_BUFFER_TYPE == 4
		TEMPLATE_BUFFERS(Acc += ByteAddress, .Load(0))
	#else
		#error Not implemented
	#endif
		DummyOutput[0] = Acc;
	}
}

#endif // SHADER_RESOURCE_TRANSITION
