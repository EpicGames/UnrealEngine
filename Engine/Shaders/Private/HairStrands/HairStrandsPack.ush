// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/PackUnpack.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair control points

FPackedHairPosition ReadPackedHairPosition(ByteAddressBuffer InBuffer, uint InPointIndex)
{
	return InBuffer.Load2(InPointIndex * FPackedHairPositionStrideInBytes);
}

void WritePackedHairPosition(RWByteAddressBuffer InBuffer, uint InPointIndex, FPackedHairPosition In)
{
	InBuffer.Store2(InPointIndex * FPackedHairPositionStrideInBytes, In);
}

void CopyPackedAttribute(inout FPackedHairPosition Out, in FPackedHairPosition In)
{
	// Copy the last 16bits which contains the CoordU|Radius|Type
	Out.y = (Out.y & 0x0000FFFF) | (In.y & 0xFFFF0000);
}

void CopyPackedHairControlPoint(RWByteAddressBuffer OutBuffer, ByteAddressBuffer InBuffer, uint Index)
{
	const uint AddressInBytes = Index * FPackedHairPositionStrideInBytes;
	OutBuffer.Store2(AddressInBytes, InBuffer.Load2(AddressInBytes));
}

struct FHairControlPoint
{
	float3 Position;
	float  WorldRadius;
	float  UCoord;
	uint   Type;
};

FPackedHairPosition PackHairControlPoint(
	FHairControlPoint CP,
	float3 InVF_PositionOffset,
	float InVF_Radius) 
{
	const uint UCoord8bits = PackR8(CP.UCoord);
	const uint Radius6bits = PackR6(CP.WorldRadius / InVF_Radius);
	const uint Type2bits   = CP.Type & 0x3;
	CP.Position -= InVF_PositionOffset;

	FPackedHairPosition Out = 0;
	Out.x = PackFloat2ToUInt(CP.Position.x, CP.Position.y);
	Out.y = f32tof16(CP.Position.z) | (UCoord8bits << 16u) | (Radius6bits << 24u) | (Type2bits << 30u);
	return Out;
}

FHairControlPoint UnpackHairControlPoint(
	FPackedHairPosition In,
	float3 InVF_PositionOffset=0,
	float InVF_Radius=1,
	float InVF_RootScale=1,
	float InVF_TipScale=1)
{
	FHairControlPoint Out = (FHairControlPoint)0;
	Out.Position	= float3(UnpackFloat2FromUInt(In.x), f16tof32(BitFieldExtractU32(In.y, 16, 0))) + InVF_PositionOffset;
	Out.UCoord		= UnpackR8(BitFieldExtractU32(In.y, 8, 16));
	Out.WorldRadius	= UnpackR6(BitFieldExtractU32(In.y, 6, 24));
	Out.Type		= BitFieldExtractU32(In.y, 2, 30);

	Out.WorldRadius *= InVF_Radius * lerp(InVF_RootScale, InVF_TipScale, Out.UCoord);
	return Out;
}

FHairControlPoint ReadHairControlPoint(
	ByteAddressBuffer InBuffer,
	uint InIndex,
	float3 InVF_PositionOffset=0,
	float InVF_Radius=1,
	float InVF_RootScale=1,
	float InVF_TipScale=1)
{
	return UnpackHairControlPoint(
		InBuffer.Load2(InIndex * FPackedHairPositionStrideInBytes),
		InVF_PositionOffset,
		InVF_Radius,
		InVF_RootScale,
		InVF_TipScale);
}

FHairControlPoint ReadHairControlPoint(
	RWByteAddressBuffer InBuffer,
	uint InIndex,
	float3 InVF_PositionOffset=0,
	float InVF_Radius=1,
	float InVF_RootScale=1,
	float InVF_TipScale=1)
{
	return UnpackHairControlPoint(
		InBuffer.Load2(InIndex * FPackedHairPositionStrideInBytes),
		InVF_PositionOffset,
		InVF_Radius,
		InVF_RootScale,
		InVF_TipScale);
}

void WriteHairControlPointPosition(
	RWByteAddressBuffer OutBuffer, 
	uint InIndex, 
	FHairControlPoint InCP,
	float3 InVF_PositionOffset=0,
	float InVF_Radius=1)
{
	OutBuffer.Store2(InIndex * FPackedHairPositionStrideInBytes, PackHairControlPoint(InCP, InVF_PositionOffset, InVF_Radius));
}

float UnpackHairControlPointCoordU(FPackedHairPosition In)
{
	return UnpackR8(BitFieldExtractU32(In.y, 8, 16));
}

float3 UnpackHairControlPointPosition(FPackedHairPosition In, float3 InVF_PositionOffset=0)
{
	return float3(UnpackFloat2FromUInt(In.x), f16tof32(BitFieldExtractU32(In.y, 16, 0))) + InVF_PositionOffset;
}

float3 ReadHairControlPointPosition(ByteAddressBuffer InBuffer, uint InPointIndex, float3 InVF_PositionOffset=0)
{
	return UnpackHairControlPointPosition(InBuffer.Load2(InPointIndex * FPackedHairPositionStrideInBytes), InVF_PositionOffset);
}

float3 ReadHairControlPointPosition(RWByteAddressBuffer InBuffer, uint InPointIndex, float3 InVF_PositionOffset=0)
{
	return UnpackHairControlPointPosition(InBuffer.Load2(InPointIndex * FPackedHairPositionStrideInBytes), InVF_PositionOffset);
}

FPackedHairPosition PackHairControlPointPosition(FPackedHairPosition InPackedData, float3 InNewPosition, float3 InPositionOffset=0)
{
	InNewPosition -= InPositionOffset;

	FPackedHairPosition Out = 0;
	Out.x = PackFloat2ToUInt(InNewPosition.x, InNewPosition.y);
	Out.y = f32tof16(InNewPosition.z) | (InPackedData.y & 0xFFFF0000);
	return Out;
}

void WritePackedHairControlPointPosition(RWByteAddressBuffer OutBuffer, uint InIndex, FPackedHairPosition InPackedData, float3 InNewPosition, float3 InPositionOffset=0)
{
	OutBuffer.Store2(InIndex * FPackedHairPositionStrideInBytes, PackHairControlPointPosition(InPackedData, InNewPosition, InPositionOffset));
}

FPackedHairPosition PackHairControlPointRadius(FPackedHairPosition InPackedData, float InNewWorldRadius, float InVFRadius)
{
	const uint Radius6bits = PackR6(InNewWorldRadius / InVFRadius);	

	FPackedHairPosition Out = InPackedData;
	Out.y = (Out.y & 0xC0FFFFFF) | (Radius6bits << 24);
	return Out;
}

bool IsValidHairStrandsSegment(const FHairControlPoint CP0, const FHairControlPoint CP1)
{
	return !(CP0.Type == HAIR_CONTROLPOINT_END && CP1.Type == HAIR_CONTROLPOINT_START);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair attributes

uint PackHairRootUV(float2 In)
{
	In.y = 1.0f - In.y;

	return	(min(uint(saturate(In.x) * 0x7FFu), 0x7FFu)      )|
			(min(uint(saturate(In.y) * 0x7FFu), 0x7FFu) << 11);
}

float2 UnpackHairRootUV(uint In)
{
	float2 RootUV, RootIndex;
	RootUV.x	= ((In) & 0x7FF) / 2047.f;			// Coord are encoded on 11 bits
	RootUV.y	= ((In >> 11) & 0x7FF) / 2047.f;	// Coord are encoded on 11 bits 
	RootIndex.x = ((In >> 22) & 0x1F);				// UDIM tile are encoded on 5bits
	RootIndex.y = ((In >> 27) & 0x1F);				// UDIM tile are encoded on 5bits

	// Invert V to compensate image origin flip. Similar to GetHairStrandsRootUV in HairCardsAttributeCommon.ush
	RootUV.y = 1.0f - RootUV.y;

	return RootUV + RootIndex;
}

float UnpackHairLength(uint In)
{
	return f16tof32(In & 0xFFFF);
}

uint PackHairSeed(float In)
{
	return PackR8(In);
}

float UnpackHairSeed(uint In)
{
	return UnpackR8(In);
}

uint PackHairClumpID(uint In)
{
	return min(In, 0xFFFFu);
}

uint3 UnpackHairClumpID(uint In)
{
	return (In & 0xFFFF).xxx;
}

uint3 UnpackHairClumpID(uint2 In)
{
	return uint3(In.x & 0xFFFF, (In.x>>16) & 0xFFFF, In.y & 0xFFFF);
}

uint PackHairColor(float3 In)
{
	return	(min(uint(In.x * 0x7FFu), 0x7FFu)    )|
			(min(uint(In.y * 0x7FFu), 0x7FFu)<<11)|
			(min(uint(In.z * 0x3FFu), 0x3FFu)<<22);
}

float3 UnpackHairColor(uint In)
{
	float3 Out;
	Out.x = (In       & 0x7FFu) / 2047.f;
	Out.y = ((In>>11) & 0x7FFu) / 2047.f;
	Out.z = ((In>>22) & 0x3FFu) / 1023.f;
	return Out;
}

uint PackHairRoughness(float In)
{
	return PackR8(In);
}

float UnpackHairRoughness(uint In)
{
	return UnpackR8(In);
}

uint PackHairAO(float In)
{
	return PackR8(In);
}

float UnpackHairAO(uint In)
{
	return UnpackR8(In);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Guide

struct FGuideCurveData
{
	uint2  CurveIndices;		// Index of the sim curve
	float2 CurveWeights;		// Weight of the sim curve
	uint2  RootPointIndices;		// Index of the first/root point of the sim curve (stored to avoid indirect fetch on CurveBuffer)
};

FGuideCurveData UnpackGuideCurveData(ByteAddressBuffer InBuffer, uint InCurveIndex, bool bUseSingleGuide)
{
	// This code assumes:
	// * HAIR_INTERPOLATION_CURVE_STRIDE == 8
	// * HAIR_INTERPOLATION_MAX_GUIDE_COUNT == 2

	FGuideCurveData Out = (FGuideCurveData)0;
	if (bUseSingleGuide)
	{
		const uint AddressInBytes = InCurveIndex * HAIR_INTERPOLATION_CURVE_STRIDE;
		const uint2 Data = InBuffer.Load2(AddressInBytes);

		Out.CurveIndices = Data.x & 0xFFFFFF;
		Out.CurveWeights = UnpackR8(Data.x>>24);
		Out.RootPointIndices = Data.y & 0xFFFFFF;
	}
	else
	{
		const uint AddressInBytes = InCurveIndex * HAIR_INTERPOLATION_CURVE_STRIDE * HAIR_INTERPOLATION_MAX_GUIDE_COUNT;
		const uint4 Data = InBuffer.Load4(AddressInBytes);

		Out.CurveIndices.x = Data.x & 0xFFFFFF;
		Out.CurveWeights.x = UnpackR8(Data.x>>24);
		Out.RootPointIndices.x = Data.y & 0xFFFFFF;

		Out.CurveIndices.y = Data.z & 0xFFFFFF;
		Out.CurveWeights.y = UnpackR8(Data.z>>24);
		Out.RootPointIndices.y = Data.w & 0xFFFFFF;
	}
	return Out;
}

struct FGuidePointData
{
	uint2  PointIndices;
	float2 PointLerps;
};

FGuidePointData UnpackGuidePointData(ByteAddressBuffer InBuffer, uint InPointIndex, bool bUseSingleGuide)
{
	// This code assumes:
	// * HAIR_INTERPOLATION_POINT_STRIDE == 2
	// * HAIR_INTERPOLATION_MAX_GUIDE_COUNT == 2

	FGuidePointData Out = (FGuidePointData)0;
	if (bUseSingleGuide)
	{
		// Ensure address is aligned on 4
		// Since the data is 16bit wide, odd index needs to be shifted out of the 32bit input
		// [    uint 32bits   ][    uint 32bits   ] ...
		// [ Point0 ][ Point1 ][ Point2 ][ Point3 ] ...
		const uint AddressInBytes = InPointIndex * HAIR_INTERPOLATION_POINT_STRIDE;
		const uint AlignedAddressInBytes = AddressInBytes & 0xFFFFFFFE;
		uint Data = InBuffer.Load(AlignedAddressInBytes);
		Data = InPointIndex & 1 ? (Data>>16u) : Data;
		
		Out.PointIndices	= Data & 0xFF;
		Out.PointLerps		= UnpackR8(Data >> 8);
	}
	else
	{
		const uint AddressInBytes = InPointIndex * HAIR_INTERPOLATION_POINT_STRIDE * HAIR_INTERPOLATION_MAX_GUIDE_COUNT;
		const uint Data = InBuffer.Load(AddressInBytes);

		Out.PointIndices.x	= Data & 0xFF;
		Out.PointLerps.x	= UnpackR8(Data >> 8);

		Out.PointIndices.y	= (Data>>16) & 0xFF;
		Out.PointLerps.y	= UnpackR8(Data >> 24);
	}
	return Out;
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cards guides

struct FCardsGuideData
{
	uint  VertexIndex;
	float VertexLerp;
};

FCardsGuideData ReadCardGuideData(ByteAddressBuffer InBuffer, uint InVertexId)
{
	const uint Data = InBuffer.Load(InVertexId * HAIR_INTERPOLATION_CARDS_GUIDE_STRIDE);

	FCardsGuideData Out = (FCardsGuideData)0;
	Out.VertexIndex	= Data & 0xFFFFFF;
	Out.VertexLerp	= UnpackR8(Data >> 24);
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Curve

struct FHairCurve
{
	uint PointIndex;
	uint PointCount;
};

uint PackHairCurve(FHairCurve In)
{
	#if HAIR_MAX_NUM_POINT_PER_CURVE != 0xFF || HAIR_MAX_NUM_POINT_PER_GROUP != 0xFFFFFF
	#error Update hair curve ENCODING_TYPE
	#endif

	return (In.PointIndex & HAIR_MAX_NUM_POINT_PER_GROUP) | (In.PointCount << 24);
}

FHairCurve UnpackHairCurve(uint In)
{
#if HAIR_MAX_NUM_POINT_PER_CURVE != 0xFF || HAIR_MAX_NUM_POINT_PER_GROUP != 0xFFFFFF
#error Update hair curve ENCODING_TYPE
#endif

	FHairCurve Out = (FHairCurve)0;
	Out.PointIndex = In & HAIR_MAX_NUM_POINT_PER_GROUP;
	Out.PointCount = In >> 24;
	return Out;
}

FHairCurve ReadHairCurve(ByteAddressBuffer InBuffer, uint InIndex)
{
	return UnpackHairCurve(InBuffer.Load(InIndex * 4u));
}

void WriteHairCurve(RWByteAddressBuffer InBuffer, uint InIndex, FHairCurve In)
{
	InBuffer.Store(InIndex * 4u, PackHairCurve(In));
}

uint ReadHairPointToCurveIndex(ByteAddressBuffer InBuffer, uint InPointIndex)
{
	// Encoding: base curve index + delta encoding for 8 consecutive control points

	const uint PointIndex8 = InPointIndex >> 3u;
	const uint LocalIndex8 = InPointIndex & 0x7;
	const uint Data = InBuffer.Load(PointIndex8 * 4u);

	// Extract the curve index delta encoding
	const uint Mask     = BitFieldMaskU32(LocalIndex8+1, 0);
	const uint Bitfield = BitFieldExtractI32(Data, 8u, 24u);
	const uint Deltas   = countbits(Mask & Bitfield);

	return (Data & 0xFFFFFF) + Deltas;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// UVs

uint PackUVs(float2 UV)
{
	return (f32tof16(UV.x) & 0xFFFF) | ((f32tof16(UV.y) & 0xFFFF) << 16);
}

float PackUVsToFloat(float2 UV)
{
	return asfloat(PackUVs(UV));
}

float2 UnPackUVs(uint InUV)
{
	float2 Out;
	Out.x = f16tof32(InUV & 0xFFFF);
	Out.y = f16tof32((InUV >> 16) & 0xFFFF);
	return Out;
}

float2 UnPackUVs(float InUV)
{
	return UnPackUVs(asuint(InUV));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Normal

uint PackVertexNormal(float3 In)
{
	return
		(PackUnorm10(In.x * 0.5f + 0.5f)    )|
		(PackUnorm10(In.y * 0.5f + 0.5f)<<10)|
		(PackUnorm10(In.z * 0.5f + 0.5f)<<20);
}

float PackVertexNormalToFloat(float3 In)
{
	return asfloat(PackVertexNormal(In));
}

float3 UnpackVertexNormal(uint In)
{
	const float3 N = float3(
		UnpackUnorm10(In    ) * 2.f - 1.f,
		UnpackUnorm10(In>>10) * 2.f - 1.f,
		UnpackUnorm10(In>>20) * 2.f - 1.f);
	return normalize(N);
}

float3 UnpackVertexNormal(float In)
{
	return UnpackVertexNormal(asuint(In));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Barycentrics

uint PackBarycentrics(float2 B)
{
	return f32tof16(B.x) | (f32tof16(B.y) << 16);
}

float3 UnpackBarycentrics(uint E)
{
	float3 Out;
	Out.x = f16tof32(E & 0xFFFF);
	Out.y = f16tof32((E >> 16) & 0xFFFF);
	Out.z = 1 - Out.x - Out.y;
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Triangle

struct FHairTriangleIndex
{
	uint TriangleIndex;
	uint SectionIndex;
};

uint PackTriangleIndex(uint TriangleIndex, uint SectionIndex)
{
	return ((SectionIndex & 0xFF)<<24) | (TriangleIndex & 0xFFFFFF);
}

FHairTriangleIndex UnpackTriangleIndex(uint Encoded)
{
	FHairTriangleIndex Out;
	Out.SectionIndex = (Encoded>>24) & 0xFF;
	Out.TriangleIndex = Encoded & 0xFFFFFF;
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Source point/curve remapping
// Translate a curve/point index into its source data index

uint ReadHairSourceCurveIndex(ByteAddressBuffer InBuffer, uint InIndex, bool bIsBufferValid)
{
	uint Out = ~0;
	if (bIsBufferValid)
	{
		Out = InBuffer.Load(InIndex * 4u);
	}
	return Out;
}

uint ReadHairSourcePointIndex(ByteAddressBuffer InBuffer, uint InIndex, bool bIsBufferValid)
{
	uint Out = ~0;
	if (bIsBufferValid)
	{
		Out = InBuffer.Load(InIndex * 4u);
	}
	return Out;
}
uint ReadHairSourceAttributeIndex(ByteAddressBuffer InBuffer, uint InIndex, bool bIsBufferValid)
{
	uint Out = ~0;
	if (bIsBufferValid)
	{
		Out = InBuffer.Load(InIndex * 4u);
	}
	return Out;
}