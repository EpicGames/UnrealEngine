// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef VOXEL_TRAVERSAL_DEBUG
#define VOXEL_TRAVERSAL_DEBUG 0
#endif
#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
//#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR

#include "../HairStrands/HairStrandsVoxelPageCommon.ush"
#include "../HairStrands/HairStrandsVoxelPageTraversal.ush"


// RayOrigin is in translated world space
float InternalTraverseHair(uint2 PixelCoord, float3 RayOrigin, float3 RayDirection, float InMinT, float InOcclusionThreshold, float3 VoxelRandom, float RayRandom)
{
	float HitT = InMinT;

	// Depth bias
	// Origin is shifted voxels away towards the light + a constant bias of the size of the voxel
	const float3 NormalizedDepthBias = (RayRandom * RayDirection + VoxelRandom);

	#if VOXEL_TRAVERSAL_DEBUG
	const bool bDebugEnabled = PixelCoord.x == GetCursorPos().x && PixelCoord.y == GetCursorPos().y;
	#else
	const bool bDebugEnabled = false;
	#endif

	FHairTraversalResult Result = InitHairTraversalResult();

	const float DistanceThreshold = 1000.0f; 
	const float CoverageThreshold = 0.995f; // When Coverage is high, we do not trace shadow on opaque since hair/fur is covering the background.

	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution = VirtualVoxel.PageCountResolution;
	CommonDesc.PageTextureResolution = VirtualVoxel.PageTextureResolution;
	CommonDesc.PageResolution = VirtualVoxel.PageResolution;
	CommonDesc.PageResolutionLog2 = VirtualVoxel.PageResolutionLog2;

	const float3 P0 = RayOrigin;
	const float3 P1 = RayOrigin + RayDirection * (HitT>=0 ? min(HitT, DistanceThreshold) : DistanceThreshold);

	for (uint MacroGroupId = 0; MacroGroupId < VirtualVoxel.NodeDescCount; ++MacroGroupId)
	{
		const FPackedVirtualVoxelNodeDesc PackedNode = VirtualVoxel.NodeDescBuffer[MacroGroupId];
		const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(PackedNode, VirtualVoxel.PageResolution);

		FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
		TraversalSettings.DensityScale = VirtualVoxel.DensityScale_Raytracing;
		TraversalSettings.CountThreshold = InOcclusionThreshold;
		TraversalSettings.DistanceThreshold = DistanceThreshold;
		TraversalSettings.bDebugEnabled = bDebugEnabled;
		TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Raytracing;
		TraversalSettings.Random = VoxelRandom;
		TraversalSettings.bUseOpaqueVisibility = false;
		TraversalSettings.bCastShadow = true;

		const float3 DepthBias = NormalizedDepthBias * NodeDesc.VoxelWorldSize;
		Result = ComputeHairCountVirtualVoxel(
			P0 + DepthBias,
			P1 + DepthBias,
			CommonDesc,
			NodeDesc,
			VirtualVoxel.PageIndexBuffer,
			VirtualVoxel.PageTexture,
			TraversalSettings);

		if (Result.HitT >= 0)
		{
			const float WorldHitT = Result.HitT;
			HitT = HitT >= 0 ? min(HitT, WorldHitT) : WorldHitT;
		}
	}

	return HitT;
}

float TraverseHair(uint2 PixelCoord, float3 RayOrigin, float3 RayDirection, float InMinT, float InOcclusionThreshold)
{
	float HitT = InMinT;

	const float3 VoxelRandom = 0.5f;
	const float  RayRandom   = 1;
	return InternalTraverseHair(PixelCoord, RayOrigin, RayDirection, InMinT, InOcclusionThreshold, VoxelRandom, RayRandom);
}

float TraverseHair(uint2 PixelCoord, inout FRandomSequence RandSequence, float3 RayOrigin, float3 RayDirection, float InMinT, float InOcclusionThreshold)
{
	float HitT = InMinT;

	// 0: No jitter, 1:Time-Random jitter, 2:Time-Constant jitter
	float3 VoxelRandom = 0.5f;
	float  RayRandom   = 1.0f;
	if (VirtualVoxel.JitterMode == 1)
	{
		const float4 Rnd = RandSequence.Get4D();
		VoxelRandom = Rnd.xyz;
		RayRandom = Rnd.w;
	}
	else
	{
		const uint2  Seed0 = Rand3DPCG16(int3(PixelCoord, 0)).xy;
		const uint2  Seed1 = Rand3DPCG16(int3(PixelCoord + 17, 0)).xy;
		const float4 Rnd = Rand16ToFloat(uint4(Seed0, Seed1));
		VoxelRandom = Rnd.xyz;
		RayRandom = Rnd.w;
	}

	return InternalTraverseHair(PixelCoord, RayOrigin, RayDirection, InMinT, InOcclusionThreshold, VoxelRandom, RayRandom);
}