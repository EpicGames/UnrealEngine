// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush" 
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_RT_GEOMETRY

// This has to match EHairStrandsRayTracingPrimitiveType enum in HairStrandsInterface.h
#define PRIMITIVE_TYPE_PROCEDURAL          0
#define PRIMITIVE_TYPE_TRIANGLE            1
#define PRIMITIVE_TYPE_LINEAR_SWEPT_SPHERE 2

uint PointCount;

float HairStrandsVF_HairRadius;
float HairStrandsVF_HairRootScale;
float HairStrandsVF_HairTipScale;

uint HairStrandsVF_RegisteredIndex;
StructuredBuffer<float4> PositionOffsetBuffer;
ByteAddressBuffer PositionBuffer;
Buffer<float4> 	 TangentBuffer;
RWBuffer<float4> OutputPositionBuffer;	
#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_PROCEDURAL
uint             RaytracingProceduralSplits;
#else
RWStructuredBuffer<uint> OutputIndexBuffer;
#endif

uint HairStrandsVF_bCullingEnable;
#if PERMUTATION_CULLING == 1
Buffer<uint>	HairStrandsVF_CullingIndirectBuffer;
Buffer<uint>	HairStrandsVF_CullingIndexBuffer;
Buffer<uint>	HairStrandsVF_CullingIndirectBufferArgs;
#endif

FHairControlPoint GetControlPoint(uint ControlPointId)
{
	FHairControlPoint Out = (FHairControlPoint)0;

	if (ControlPointId < PointCount)
	{
		Out = ReadHairControlPoint(
			PositionBuffer, 
			ControlPointId,
			ReadRenPositionOffset(PositionOffsetBuffer, HairStrandsVF_RegisteredIndex),
			HairStrandsVF_HairRadius,
			HairStrandsVF_HairRootScale,
			HairStrandsVF_HairTipScale);
	}

	return Out;
}

float3x3 GetTangentBasis(uint ControlPointId)
{
	float3 Tx = TangentBuffer[2 * ControlPointId    ].xyz;
	float3 Tz = TangentBuffer[2 * ControlPointId + 1].xyz;
	float3 Ty = cross(Tz, Tx);
	return float3x3(Tx, Ty, Tz);
}

#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_PROCEDURAL
void SetAABB(uint InVertexIndex, float3 Lo, float3 Hi)
{
	OutputPositionBuffer[2 * InVertexIndex + 0] = float4(Lo.x, Lo.y, Lo.z, Hi.x);
	OutputPositionBuffer[2 * InVertexIndex + 1] = float4(Hi.y, Hi.z, 0.0, 0.0);
}
#endif

// a whole segment of invalid values
void SetInvalidSegment(uint InVertexIndex)
{
#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_PROCEDURAL
	const float NaN = asfloat(0x7FC00000);
	for (uint i = 0; i < RaytracingProceduralSplits; i++)
		SetAABB(RaytracingProceduralSplits * InVertexIndex + i, NaN, NaN);
#elif PRIMITIVE_TYPE == PRIMITIVE_TYPE_LINEAR_SWEPT_SPHERE
	const uint BasePoint = InVertexIndex * 2;
	OutputIndexBuffer[BasePoint    ] = 0;
	OutputIndexBuffer[BasePoint + 1] = 0;
#else // PRIMITIVE_TYPE == PRIMITIVE_TYPE_TRIANGLE
	const uint BasePoint = InVertexIndex * 4;
	const uint TrisPerSegment = 8;
	const uint IdxsPerSegment = TrisPerSegment * 3;
	const uint BaseOutIndex = InVertexIndex * IdxsPerSegment;
	UNROLL
	for (uint Index = 0; Index < IdxsPerSegment; Index++)
	{
		OutputIndexBuffer[BaseOutIndex + Index] = BasePoint;
	}
#endif
}

// TODO: this is not used at the moment
void SetCapSegment(uint InVertexIndex)
{
#if 1
	SetInvalidSegment(InVertexIndex);
#else
	const uint BasePoint = InVertexIndex * 4;
	const uint TrisPerSegment = 8;
	const uint IdxsPerSegment = TrisPerSegment * 3;
	const uint BaseOutIndex = InVertexIndex * IdxsPerSegment;
	OutputIndexBuffer[BaseOutIndex + 0] = BasePoint;
	OutputIndexBuffer[BaseOutIndex + 1] = BasePoint + 1;
	OutputIndexBuffer[BaseOutIndex + 2] = BasePoint + 2;

	OutputIndexBuffer[BaseOutIndex + 3] = BasePoint;
	OutputIndexBuffer[BaseOutIndex + 4] = BasePoint + 2;
	OutputIndexBuffer[BaseOutIndex + 5] = BasePoint + 3;

	// fill the reset with degenerate triangles
	UNROLL
	for (uint Index = 6; Index < IdxsPerSegment; Index++)
	{
		OutputIndexBuffer[BaseOutIndex + Index] = BasePoint;
	}
#endif
}

#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_TRIANGLE
void SetBodySegment(uint InVertexIndex)
{
	const uint BotPoint = InVertexIndex * 4;
	const uint TopPoint = BotPoint + 4;

	const uint TrisPerSegment = 8;
	const uint IdxsPerSegment = TrisPerSegment * 3;
	const uint BaseOutIndex = InVertexIndex * IdxsPerSegment;

	UNROLL
	for (uint PrevIdx = 3, Idx = 0, OutIndex = BaseOutIndex; Idx < 4; PrevIdx = Idx, Idx++, OutIndex += 6)
	{
		// Note that this vertex ordering is carefully chosen to work with the Vertex Factory in RT mode
		// In particular, we _must_ put one of the bottom point as the last vertex of the triangle
		// so that the interpolation step will copy the right HairControlPointId onwards
		OutputIndexBuffer[OutIndex + 0] = TopPoint + PrevIdx;
		OutputIndexBuffer[OutIndex + 1] = TopPoint + Idx;
		OutputIndexBuffer[OutIndex + 2] = BotPoint + PrevIdx;

		OutputIndexBuffer[OutIndex + 3] = TopPoint + Idx;
		OutputIndexBuffer[OutIndex + 4] = BotPoint + Idx;
		OutputIndexBuffer[OutIndex + 5] = BotPoint + PrevIdx; 
	}
}
#endif


float GetAABBHalfAreaForSegment(float4 A, float4 B)
{
	float3 BoxLo = min(A.xyz - A.w, B.xyz - B.w);
	float3 BoxHi = max(A.xyz + A.w, B.xyz + B.w);
	float3 BoxDiag = BoxHi - BoxLo;
	return dot(BoxDiag.xyz, BoxDiag.yzx);
}

bool IsValidPosition(FHairControlPoint P)
{
	return all(abs(P.Position) < INFINITE_FLOAT);
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
#if PERMUTATION_CULLING == 1
	const uint MaxPointCount = HairStrandsVF_bCullingEnable ? HairStrandsVF_CullingIndirectBuffer[3] : PointCount;
#else
	const uint MaxPointCount = PointCount;
#endif

#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_LINEAR_SWEPT_SPHERE
	// preserve the first entry for Nan to indicate invalid hair strand
	const uint VertexBufferIndexShift = 1;
	
	if (DispatchThreadId < VertexBufferIndexShift)
	{
		const float NaN = asfloat(0x7FC00000);
		OutputPositionBuffer[DispatchThreadId] = NaN;
	}

	if (DispatchThreadId < MaxPointCount)
	{
		const FHairControlPoint CP = GetControlPoint(DispatchThreadId);
		OutputPositionBuffer[DispatchThreadId + VertexBufferIndexShift] = float4(CP.Position, CP.WorldRadius);
	}
#endif

	uint VertexIndex0 = DispatchThreadId;
	uint VertexIndex1 = VertexIndex0 + 1;

	const bool bIsValid = VertexIndex0 < PointCount && VertexIndex1 < PointCount;
	if (!bIsValid)
	{
		SetInvalidSegment(VertexIndex0);
		return;
	}

	uint SourceIndex0 = VertexIndex0;
	uint SourceIndex1 = VertexIndex1;
#if PERMUTATION_CULLING == 1
	if (HairStrandsVF_bCullingEnable)
	{
		if (VertexIndex1 >= MaxPointCount)
		{
			SetInvalidSegment(VertexIndex0);
			return;
		}

		SourceIndex0 = HairStrandsVF_CullingIndexBuffer[VertexIndex0];
		SourceIndex1 = HairStrandsVF_CullingIndexBuffer[VertexIndex1];
	}
#endif

	FHairControlPoint P0 = GetControlPoint(SourceIndex0);
#if PRIMITIVE_TYPE == PRIMITIVE_TYPE_PROCEDURAL 
	if (P0.Type == HAIR_CONTROLPOINT_END || !IsValidPosition(P0))
	{
		// this vertex doesn't map to a valid segment
		SetInvalidSegment(VertexIndex0);
	}
	else
	{
		// fetch the other point and create the aabb around the line
		FHairControlPoint P1 = GetControlPoint(SourceIndex1);
		if (!IsValidPosition(P1))
		{
			SetInvalidSegment(VertexIndex0);
		}
		else
		{
			float4 A = float4(P0.Position, P0.WorldRadius);
			float4 B = float4(P1.Position, P1.WorldRadius);
			if ((A.w == 0 && B.w == 0) || length2(B.xyz - A.xyz) == 0 || any(!IsFinite(A)) || any(!IsFinite(B)))
			{
				// this segment is degenerate, don't bother adding it
				SetInvalidSegment(VertexIndex0);
			}
			else
			{
				// measure aabb surface area prior to splitting
				float FullArea = GetAABBHalfAreaForSegment(A, B);

				float4 SplitA = A;
				float SumArea = 0.0;
				for (uint i = 0; i < RaytracingProceduralSplits; i++)
				{
					float4 SplitB = lerp(A, B, float(i + 1) / float(RaytracingProceduralSplits));
					float3 Lo = min(SplitA.xyz - SplitA.w, SplitB.xyz - SplitB.w);
					float3 Hi = max(SplitA.xyz + SplitA.w, SplitB.xyz + SplitB.w);
					SumArea += GetAABBHalfAreaForSegment(SplitA, SplitB);
					SetAABB(RaytracingProceduralSplits * VertexIndex0 + i, Lo, Hi);
					SplitA = SplitB;
				}

				#if 1
				if (FullArea <= 2 * SumArea)
				{
					// not worth splitting? use full bbox and make other splits degenerate
					float3 Lo = min(A.xyz - A.w, B.xyz - B.w);
					float3 Hi = max(A.xyz + A.w, B.xyz + B.w);
					SetAABB(RaytracingProceduralSplits * VertexIndex0 + 0, Lo, Hi);
					for (uint i = 1; i < RaytracingProceduralSplits; i++)
					{
						const float NaN = asfloat(0x7FC00000);
						SetAABB(RaytracingProceduralSplits * VertexIndex0 + i, NaN, NaN);
					}
				}
				#endif
			}
		}
	}
#elif PRIMITIVE_TYPE == PRIMITIVE_TYPE_LINEAR_SWEPT_SPHERE
	if (P0.Type == HAIR_CONTROLPOINT_END || !IsValidPosition(P0))
	{
		// this vertex doesn't map to a valid segment
		SetInvalidSegment(VertexIndex0);
	}
	else
	{
		// fetch the other point and create the aabb around the line
		FHairControlPoint P1 = GetControlPoint(SourceIndex1);
		if (!IsValidPosition(P1) || VertexIndex1 >= MaxPointCount)
		{
			SetInvalidSegment(VertexIndex0);
		}
		else
		{
			float4 A = float4(P0.Position, P0.WorldRadius);
			float4 B = float4(P1.Position, P1.WorldRadius);
			if ((A.w == 0 && B.w == 0) || length2(B.xyz - A.xyz) == 0 || any(!IsFinite(A)) || any(!IsFinite(B)))
			{
				// this segment is degenerate, don't bother adding it
				SetInvalidSegment(VertexIndex0);
			}
			else
			{
				const uint BaseOutIndex = VertexIndex0 * 2;
				OutputIndexBuffer[BaseOutIndex    ] = SourceIndex0 + VertexBufferIndexShift;
				OutputIndexBuffer[BaseOutIndex + 1] = SourceIndex1 + VertexBufferIndexShift;
			}
		}
	}
#else // PRIMITIVE_TYPE == PRIMITIVE_TYPE_TRIANGLE
	// expand current vertex into 4 points around the curve
	const float3x3 TangentBasis = GetTangentBasis(SourceIndex0);
	const float3 N = TangentBasis[0];
	const float3 B = TangentBasis[1];

	const uint BaseOutIndex = VertexIndex0 * 4;
	{
		const float3 Px0 = P0.Position - N * P0.WorldRadius;
		const float3 Px1 = P0.Position + N * P0.WorldRadius;
		const float3 Py0 = P0.Position - B * P0.WorldRadius;
		const float3 Py1 = P0.Position + B * P0.WorldRadius;

		OutputPositionBuffer[BaseOutIndex    ] = float4(Px0, 1);
		OutputPositionBuffer[BaseOutIndex + 1] = float4(Py0, 1);
		OutputPositionBuffer[BaseOutIndex + 2] = float4(Px1, 1);
		OutputPositionBuffer[BaseOutIndex + 3] = float4(Py1, 1);
	}
	if (P0.Type == HAIR_CONTROLPOINT_END || !IsValidPosition(P0))
	{
		// last vertex of a curve, we could create a cap, but this would complicate the logic in the vertex factory
		// ignore this for now on the assumption that most hairs will have a small tip width
		SetInvalidSegment(VertexIndex0);
	}
	else
	{
		FHairControlPoint P1 = GetControlPoint(SourceIndex1);
		// Skip segment if second vertex is degenerate, or we have reached the very last vertex
		// TODO: is there an off-by-one error elsewhere? the last point seems to always be near the origin for some reason (but only in this mode)
		// TODO: Check if this issue impacts the other modes as well
		if (!IsValidPosition(P1) || SourceIndex1 == (MaxPointCount - 1))
		{
			SetInvalidSegment(VertexIndex0);
		}
		else
		{
			// regular point, build the body of the cylinder
			SetBodySegment(VertexIndex0);
		}
	}
#endif
}

#endif // SHADER_RT_GEOMETRY

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_POSITION_CHANGED

#include "../ShaderPrint.ush"
#include "HairStrandsAABBCommon.ush"

#if GROUP_SIZE != 1024
#error GROUP_SIZE has changed, please update the reduction code.
#endif

uint InstanceResgisteredIndex;
uint PointCount;
uint bDrawInvalidElement;
float PositionThreshold2;

uint HairStrandsVF_bCullingEnable;
Buffer<uint> 	 HairStrandsVF_CullingIndirectBuffer;
Buffer<uint> 	 HairStrandsVF_CullingIndexBuffer;
ByteAddressBuffer CurrPositionBuffer;
ByteAddressBuffer PrevPositionBuffer;
Buffer<int> 	 GroupAABBBuffer;
RWBuffer<uint> 	 InvalidationBuffer;
RWBuffer<uint> 	 InvalidationPrintCounter;

groupshared uint s_InvalidMask[GROUP_SIZE];

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint LinearIndex : SV_GroupIndex)
{
	// Early out: avoid full comparison if previous groom have already invalidated the frame
	const bool bIsAlreadyInvalid = InvalidationBuffer[0] > 0;
	s_InvalidMask[LinearIndex] = 0;

	if (!bIsAlreadyInvalid)
	{
		// Fetch vertex index if culling is enabled
		uint VertexIndex0 = DispatchThreadId.x;
		bool bIsValid = VertexIndex0 < PointCount;
		if (bIsValid && HairStrandsVF_bCullingEnable)
		{
			const uint VertexCountAfterCulling = HairStrandsVF_CullingIndirectBuffer[3];
			const uint FetchIndex0 = DispatchThreadId.x;
			bIsValid = FetchIndex0 < VertexCountAfterCulling;
			if (bIsValid)
			{
				VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
			}
		}

		// Position comparison
		if (bIsValid)
		{
			const FHairControlPoint Curr = ReadHairControlPoint(CurrPositionBuffer, VertexIndex0, float3(0, 0, 0), 1, 1, 1);
			const FHairControlPoint Prev = ReadHairControlPoint(PrevPositionBuffer, VertexIndex0, float3(0, 0, 0), 1, 1, 1);
			const float3 Diff = Curr.Position - Prev.Position;
			const bool bIsInvalid = dot(Diff, Diff) > PositionThreshold2;

			// Draw red bounding box around the groom having some changed position 
			if (bIsInvalid && bDrawInvalidElement)
			{
				uint Offset = 0;
				InterlockedAdd(InvalidationPrintCounter[0], 1, Offset);
				if (Offset < 2)
				{
					const FHairAABB Bound = ReadHairAABB(InstanceResgisteredIndex, GroupAABBBuffer);
					AddAABBTWS(Bound.Min, Bound.Max, ColorRed);
				}
			}
			s_InvalidMask[LinearIndex] = bIsInvalid ? 1u : 0u;
		}
	}

	// Reduction
	if (LinearIndex < 512)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 512];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 256)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 256];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 128)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 128];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 64)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 64];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 32)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 8)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		s_InvalidMask[LinearIndex] = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 1)
	{
		const uint InvalidMask = s_InvalidMask[LinearIndex] | s_InvalidMask[LinearIndex + 1];
		if (InvalidMask > 0)
		{
			InterlockedOr(InvalidationBuffer[0], 1u);
		}
	}
}

#endif // SHADER_POSITION_CHANGED
