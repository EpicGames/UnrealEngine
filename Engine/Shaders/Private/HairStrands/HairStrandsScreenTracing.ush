// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../BlueNoise.ush"
#include "../SSRT/SSRTRayCast.ush"
#include "../HZB.ush"

float ScreenTraceLength;

float GetRayVisibility(float3 TranslatedWorldPosition,float SceneDepth, float3 RayDirection, uint2 PixelCoord, uint PixelRayIndex, uint NumPixelSamples, bool bJitter)
{
	const float SlopeCompareToleranceScale = 0.5f;
	const float MaxScreenTraceFraction = ScreenTraceLength * 2.0f / (float)View.ViewSizeAndInvSize.x;
	float TraceDistance = MaxScreenTraceFraction * 2.0 * GetScreenRayLengthMultiplierForProjectionType(SceneDepth).x;
	float DepthThresholdScale = View.ProjectionDepthThicknessScale;

	const float PositionBias = 0.1f;
	const float3 TranslatedWorldStartPosition = TranslatedWorldPosition + RayDirection * PositionBias;
	
	FSSRTCastingSettings CastSettings = CreateDefaultCastSettings();
	CastSettings.bStopWhenUncertain = true;

	bool bHit = false;
	float Level;
	float3 HitUVz;
	bool bRayWasClipped;
	uint NumSteps = 4;
	float StartMipLevel = 0;
	float RayRoughness = .2f;
	uint2 NoiseCoord = PixelCoord * uint2(NumPixelSamples, 1) + uint2(PixelRayIndex, 0);
	float StepOffset = bJitter > 0 ? BlueNoiseScalar(PixelCoord, View.StateFrameIndex) : 0;

	FSSRTRay Ray = InitScreenSpaceRayFromWorldSpace(
		TranslatedWorldStartPosition, RayDirection,
		/* WorldTMax = */ TraceDistance,
		/* SceneDepth = */ SceneDepth,
		/* SlopeCompareToleranceScale */ SlopeCompareToleranceScale * DepthThresholdScale * (float)NumSteps,
		/* bExtendRayToScreenBorder = */ false,
		/* out */ bRayWasClipped);

	bool bUncertain;
	float3 DebugOutput;

	CastScreenSpaceRay(
		ClosestHZBTexture, ClosestHZBTextureSampler,
		StartMipLevel,
		CastSettings,
		Ray, RayRoughness, NumSteps, StepOffset - .9f,
		HZBUvFactorAndInvFactor, false,
		/* out */ DebugOutput,
		/* out */ HitUVz,
		/* out */ Level,
		/* out */ bHit,
		/* out */ bUncertain);

	bHit = bHit && !bUncertain;
	return bHit ? 0.0f : 1.0f;
}

float ComputeScreenTraceHairCount(
	const uint2 PixelCoord,
	const float PixelRadius,
	const float SceneDepth,
	float3 TranslatedWorldPosition,
	float3 SampleDirection,
	uint SampleIt=0,
	uint SampleCount=1,
	bool bJitter = true)
{
	const float HairCountPerScreenSpaceIntersection = 1.5f; // Arbitrary value, visually tweaked
	const float ScreenTraceVis = GetRayVisibility(TranslatedWorldPosition, SceneDepth, SampleDirection, PixelCoord, SampleIt, SampleCount, bJitter);
	return saturate(1.f-ScreenTraceVis) * HairCountPerScreenSpaceIntersection;
}
