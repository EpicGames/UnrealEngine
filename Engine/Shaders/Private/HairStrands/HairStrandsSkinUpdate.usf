// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush"
#include "../GpuSkinCommon.ush"

#if SHADER_SKIN_UPDATE
//----------------------------------------------------------------------------------

uint bWriteTangent;
uint WeightIndexSize; // bone index size: 1 - 8bit or 2 - 16bit
uint SectionVertexBaseIndex;
uint NumTotalVertices;
uint NumVertexToProcess;
uint WeightStride;
uint TangentFormat;

// Bone transform offset (supports both per-section buffers and shared scene skinning buffer).
// When using per-section bone buffers (skin cache), offset is 0.
// When using the scene skinning extension's shared buffer, offset indexes into the correct region.
uint BoneTransformOffset;
uint PrevBoneTransformOffset;

Buffer<uint> WeightLookup;
Buffer<uint> VertexWeights;
Buffer<float> RestPositions;
Buffer<SNORM float4> RestTangents;

// Current Position
FCompressedBoneTransformBuffer BoneMatrices;
RWBuffer<float> DeformedPositions;
RWBuffer<TANGENT_RWBUFFER_FORMAT> DeformedTangents; // GL does not support SNORM images, so instead we write to SINT

// Previous Position
#if PERMUTATION_PREV
FCompressedBoneTransformBuffer PrevBoneMatrices;
RWBuffer<float> PrevDeformedPositions;
#endif

FComputeBoneMatrixInputs MakeBoneInputs(FCompressedBoneTransformBuffer InBoneMatrices, uint InBoneTransformOffset)
{
	FComputeBoneMatrixInputs Inputs;
	Inputs.BoneMatrices = InBoneMatrices;
	Inputs.BoneTransformOffset = InBoneTransformOffset;
	Inputs.SectionBoneOffset = 0;
	Inputs.TransformStorageMode = GPUSKIN_TRANSFORM_STORAGE_BONE_MAP;
	return Inputs;
}

//----------------------------------------------------------------------------------

[numthreads(GROUP_SIZE,1,1)]
void UpdateSkinPositionCS(uint3 GroupID : SV_GroupID,
 uint3 DispatchThreadID : SV_DispatchThreadID,
 uint3 GroupThreadID : SV_GroupThreadID)
{
	const uint VertexIndex = DispatchThreadID.x + SectionVertexBaseIndex;
	if (DispatchThreadID.x < NumVertexToProcess && VertexIndex < NumTotalVertices)
	{
		const uint PositionOffset = VertexIndex * 3;
		const uint WeightOffset = VertexIndex * WeightStride / 4;

		#if !GPUSKIN_UNLIMITED_BONE_INFLUENCE
		const FGPUSkinIndexAndWeight IndicesAndWeights = GetBlendIndicesAndWeights(VertexWeights, WeightOffset);
		#endif

		const float4 InputPosition = float4(
			RestPositions[PositionOffset],
			RestPositions[PositionOffset+1],
			RestPositions[PositionOffset+2],
			1.0);

		// Current position
		{
			FComputeBoneMatrixInputs BoneInputs = MakeBoneInputs(BoneMatrices, BoneTransformOffset);

			#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
			const FBoneMatrix BlendedMatrix = ComputeBoneMatrixWithUnlimitedInfluences(BoneInputs, VertexWeights, WeightIndexSize, WeightLookup[VertexIndex]);
			#else
			const FBoneMatrix BlendedMatrix = ComputeBoneMatrixWithLimitedInfluences(BoneInputs, IndicesAndWeights);
			#endif

			const float3 OutputPosition =  mul(BlendedMatrix, InputPosition);
			DeformedPositions[PositionOffset  ] = OutputPosition.x;
			DeformedPositions[PositionOffset+1] = OutputPosition.y;
			DeformedPositions[PositionOffset+2] = OutputPosition.z;

			if (bWriteTangent)
			{
				const float4 InputTangentZ = TangentBias_SkinCache(RestTangents[2 * VertexIndex + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z]);

				// Use similar code than GPU skin cache, but this should use the inverse transpose for correct normal stransformation
				const float3 OutputTangentZ = mul(BlendedMatrix, float4(InputTangentZ.xyz,0)).xyz;
			
				DeformedTangents[VertexIndex * 2 + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z] = TangentUnbias_SkinCache(float4(OutputTangentZ, InputTangentZ.w));
			}
		}

		// Previous position
	#if PERMUTATION_PREV
		{
			FComputeBoneMatrixInputs PrevBoneInputs = MakeBoneInputs(PrevBoneMatrices, PrevBoneTransformOffset);

			#if GPUSKIN_UNLIMITED_BONE_INFLUENCE
			const FBoneMatrix PrevBlendedMatrix = ComputeBoneMatrixWithUnlimitedInfluences(PrevBoneInputs, VertexWeights, WeightIndexSize, WeightLookup[VertexIndex]);
			#else
			const FBoneMatrix PrevBlendedMatrix = ComputeBoneMatrixWithLimitedInfluences(PrevBoneInputs, IndicesAndWeights);
			#endif

			const float3 OutputPosition = mul(PrevBlendedMatrix, InputPosition);
			PrevDeformedPositions[PositionOffset  ] = OutputPosition.x;
			PrevDeformedPositions[PositionOffset+1] = OutputPosition.y;
			PrevDeformedPositions[PositionOffset+2] = OutputPosition.z;
		}
	#endif
	}
}
#endif // SHADER_SKIN_UPDATE
