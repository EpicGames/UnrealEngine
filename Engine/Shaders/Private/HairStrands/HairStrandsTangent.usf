// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "HairStrandsPack.ush"
#include "HairStrandsClusterCommon.ush"

#if SHADER_TANGENT
uint PointCount;

ByteAddressBuffer PositionBuffer;
RWBuffer<float4> OutputTangentBuffer;

uint			HairStrandsVF_bCullingEnable;

#if PERMUTATION_CULLING == 1
Buffer<uint>	HairStrandsVF_CullingIndirectBuffer;
Buffer<uint>	HairStrandsVF_CullingIndexBuffer;
Buffer<uint>	IndirectBufferArgs;
#endif

float4 PackNormal(float3 V, uint A)
{
	return float4(clamp(V,float(-1).xxx,float(1).xxx), 1.0f);
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const int MaxPointCount = int(PointCount);
	int IndexCurr = DispatchThreadId;
	int IndexPrev = clamp(IndexCurr-1, 0, MaxPointCount-1);
	int IndexNext = clamp(IndexCurr+1, 0, MaxPointCount-1);

	bool bPrevValid = (IndexCurr-1) >= 0;
	bool bNextValid = (IndexCurr+1) < MaxPointCount;

	bool bValid = IndexCurr < MaxPointCount;
#if PERMUTATION_CULLING
	if (HairStrandsVF_bCullingEnable)
	{
		const int FetchIndex = DispatchThreadId;
		const int PointCountAfterCulling = clamp((int)HairStrandsVF_CullingIndirectBuffer[3], 0, MaxPointCount);
		bValid = false;
		if (FetchIndex < PointCountAfterCulling)
		{
			const int FetchIndexPrev = clamp(FetchIndex-1, 0, PointCountAfterCulling-1);
			const int FetchIndexCurr = clamp(FetchIndex  , 0, PointCountAfterCulling-1);
			const int FetchIndexNext = clamp(FetchIndex+1, 0, PointCountAfterCulling-1);

			IndexPrev = HairStrandsVF_CullingIndexBuffer[FetchIndexPrev];
			IndexCurr = HairStrandsVF_CullingIndexBuffer[FetchIndexCurr];
			IndexNext = HairStrandsVF_CullingIndexBuffer[FetchIndexNext];

			bPrevValid = FetchIndex-1 >= 0;
			bNextValid = FetchIndex+1 < PointCountAfterCulling;

			bValid = true;
		}
	}
#endif

	if (bValid && IndexCurr < MaxPointCount)
	{
		const FHairControlPoint CPCurr = ReadHairControlPoint(PositionBuffer,IndexCurr);
		const FHairControlPoint CPPrev = ReadHairControlPoint(PositionBuffer,IndexPrev);
		const FHairControlPoint CPNext = ReadHairControlPoint(PositionBuffer,IndexNext);

		float3 T0 = 0;
		const bool bIsPrevValid = bPrevValid && (CPPrev.Type == HAIR_CONTROLPOINT_INSIDE || CPPrev.Type == HAIR_CONTROLPOINT_START);
		if (bIsPrevValid)
		{
			T0 = normalize(CPCurr.Position - CPPrev.Position);
		}

		float3 T1 = 0;
		const bool bIsNextValid = bNextValid && (CPNext.Type == HAIR_CONTROLPOINT_INSIDE || CPNext.Type == HAIR_CONTROLPOINT_END);
		if (bIsNextValid)
		{
			T1 = normalize(CPNext.Position - CPCurr.Position);
		}

		const float3 T = normalize(T0 + T1);

		// TODO: switch to rotation minimizing frames to avoid the discontinuity
		float3x3 Basis = GetTangentBasis(T);

		// Tangent
		OutputTangentBuffer[IndexCurr*2]   = PackNormal(Basis[0], 0);
		OutputTangentBuffer[IndexCurr*2+1] = PackNormal(Basis[2], 127);

		// When using culling, active point might not be consecutive. This is an issue during material evaluation asonly the resolved segment's 
		// point ID is stored per pixel. The knowledge of the next 'valid' point ID is lost. To ensure corrent tangent interpolation during material 
		// evaluation, the adjacent point's tangent is updated with the current point value
		#if PERMUTATION_CULLING
		if (IndexNext != IndexCurr+1 && IndexCurr+1 < PointCount && CPCurr.Type != HAIR_CONTROLPOINT_END)
		{
			const uint CurrNextIndex = IndexCurr + 1;
			OutputTangentBuffer[CurrNextIndex*2]   = PackNormal(Basis[0], 0);
			OutputTangentBuffer[CurrNextIndex*2+1] = PackNormal(Basis[2], 127);
		}
		#endif
	}
}
#endif // SHADER_TANGENT