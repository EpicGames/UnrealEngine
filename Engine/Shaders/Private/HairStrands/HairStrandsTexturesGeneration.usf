// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush" 
#include "../MonteCarlo.ush"

#include "HairStrandsVisibilityCommon.ush"
#include "HairCardsTextureCommon.ush"

#define DEBUG_ENABLE 0
#if DEBUG_ENABLE
#include "../ShaderPrint.ush"
#endif

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers

#if HAIR_CARDS_MAX_TEXTURE_COUNT != 6
#error Max cards texture count is != 6. This codes needs to be updated.
#endif

void AssignValue1(inout float4 OutTexture, uint ChannelIndex, float Value)
{
	switch (ChannelIndex)
	{
		case 0: OutTexture.x = Value; break;
		case 1: OutTexture.y = Value; break;
		case 2: OutTexture.z = Value; break;
		case 3: OutTexture.w = Value; break;
		default: break;
	}
}

void AssignValue2(inout float4 OutTexture, uint ChannelIndex, float2 Value)
{
	switch (ChannelIndex)
	{
		case 0: OutTexture.xy = Value; break;
		case 1: OutTexture.zw = Value; break;
		default: break;
	}
}

#define WriteValue1(InLayout, InAttribute, In)\
{ \
	if (IsHairCardTextureIndexValid(InLayout, InAttribute))\
	{\
		const uint TexIndex = GetHairCardTextureIndex(InLayout, InAttribute);\
		const uint ChannelIndex = GetHairCardChannelIndex(InLayout, InAttribute);\
		switch (TexIndex)\
		{ \
			case 0: AssignValue1(OutTexture0, ChannelIndex, In); break;\
			case 1: AssignValue1(OutTexture1, ChannelIndex, In); break;\
			case 2: AssignValue1(OutTexture2, ChannelIndex, In); break;\
			case 3: AssignValue1(OutTexture3, ChannelIndex, In); break;\
			case 4: AssignValue1(OutTexture4, ChannelIndex, In); break;\
			case 5: AssignValue1(OutTexture5, ChannelIndex, In); break;\
		}\
	}\
}
#define WriteValue2(InLayout, InAttribute, In)\
{\
	if (IsHairCardTextureIndexValid(InLayout, InAttribute))\
	{\
		const uint TexIndex = GetHairCardTextureIndex(InLayout, InAttribute);\
		const uint ChannelIndex = GetHairCardChannelIndex(InLayout, InAttribute);\
		switch (TexIndex)\
		{\
			case 0: AssignValue2(OutTexture0, ChannelIndex, In); break;\
			case 1: AssignValue2(OutTexture1, ChannelIndex, In); break;\
			case 2: AssignValue2(OutTexture2, ChannelIndex, In); break;\
			case 3: AssignValue2(OutTexture3, ChannelIndex, In); break;\
			case 4: AssignValue2(OutTexture4, ChannelIndex, In); break;\
			case 5: AssignValue2(OutTexture5, ChannelIndex, In); break;\
		}\
	}\
}
#define WriteValue3(InLayout, InAttribute, In)\
{\
	if (IsHairCardTextureIndexValid(InLayout, InAttribute))\
	{\
	const uint TexIndex = GetHairCardTextureIndex(InLayout, InAttribute);\
		switch (TexIndex)\
		{\
			case 0: OutTexture0.xyz = In; break;\
			case 1: OutTexture1.xyz = In; break;\
			case 2: OutTexture2.xyz = In; break;\
			case 3: OutTexture3.xyz = In; break;\
			case 4: OutTexture4.xyz = In; break;\
			case 5: OutTexture5.xyz = In; break;\
		}\
	}\
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_VERTEX

int2 OutputResolution;
uint VertexCount;
uint PrimitiveCount;

uint UVsChannelIndex;
uint UVsChannelCount;

Buffer<float> VertexBuffer;
Buffer<float4> NormalsBuffer;
Buffer<float2> UVsBuffer;

void MainVS(
	uint VertexId			: SV_VertexID,
	out float4 UVPosition	: SV_POSITION,
	out float2 OutUV		: MESH_UV, 
	out float3 OutPosition	: MESH_POSITION,
	out float3 OutTangent	: MESH_TANGENT,
	out float3 OutBitangent	: MESH_BITANGENT,
	out float3 OutNormal	: MESH_NORMAL)
{
	const float3 Position = float3(
		VertexBuffer.Load(VertexId*3 + 0),
		VertexBuffer.Load(VertexId*3 + 1),
		VertexBuffer.Load(VertexId*3 + 2));

	float2 UVs = UVsBuffer.Load(VertexId * UVsChannelCount + UVsChannelIndex);
	UVs.y = 1 - UVs.y;

	UVPosition	= float4(UVs *2-1, 0.5f, 1);
	OutUV		= UVs;
	OutPosition = Position;

	const float4 VY = NormalsBuffer.Load(VertexId * 2);
	const float4 VZ = NormalsBuffer.Load(VertexId * 2 + 1);

	OutTangent  = cross(VY.xyz, VZ.xyz) * VZ.w;
	OutBitangent= VY.xyz;
	OutNormal	= VZ.xyz;
}

#endif // VERTEXSHADER

#if SHADER_PIXEL
void MainPS(
	in float4  SvPosition	: SV_Position,
	in float2  InUV			: MESH_UV,
	in float3  InPosition	: MESH_POSITION,
	in float3  InTangent	: MESH_TANGENT,
	in float3  InBitangent	: MESH_BITANGENT,
	in float3  InNormal		: MESH_NORMAL,
	out float4 OutPosition  : SV_Target0,
	out float4 OutTangent   : SV_Target1,
	out float4 OutBitangent : SV_Target2,
	out float4 OutNormal    : SV_Target3,
	out float2 OutUV        : SV_Target4,
	out uint   OutTriangleMask : SV_Target5)
{
	const float3 T = normalize(InTangent);
	const float3 B = normalize(InBitangent);
	const float3 N = normalize(InNormal);

	OutPosition     = float4(InPosition,1);
	OutTangent      = float4(InTangent,1);
	OutBitangent    = float4(InBitangent,1);
	OutNormal       = float4(InNormal,1);
	OutUV           = InUV;
	OutTriangleMask = 1;
}
#endif //SHADER_PIXEL

#if SHADER_COMPUTE 

#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsAttributeCommon.ush"
#include "HairCardsAttributeCommon.ush"

HAIR_STRANDS_INSTANCE_PARAMETERS(InVF)

#include "HairStrandsAttributeCommon.ush"
#define HAIR_STRANDS_ATTRIBUTE_ACCESSORS(Name) InVF_##Name
#include "/Engine/Private/HairStrands/HairStrandsAttributeTemplate.ush"

float3 ComputeTangent(
	const FHairControlPoint CP0,
	const FHairControlPoint CP1,
	const FHairControlPoint CP2)
{
	const float3 T0 = normalize(CP1.Position - CP0.Position);
	const float3 T1 = normalize(CP2.Position - CP1.Position);
	return normalize(T0 + T1);
}

float3 ComputeTangent(
	const FHairControlPoint CP0,
	const FHairControlPoint CP1)
{
	return normalize(CP1.Position - CP0.Position);
}

int2 TileSize;
int2 TileOffsetInPixels;
uint LayoutIndex;
int2 OutputResolution;
uint VertexCount;
uint PrimitiveCount;
uint bHasMaterialData;

Buffer<float> VertexBuffer;
Buffer<float4> NormalsBuffer;
Buffer<float2> UVsBuffer;

float MaxDistance;
int TracingDirection;

float3 Voxel_MinBound;
float3 Voxel_MaxBound;
int3 Voxel_Resolution;
float Voxel_Size;
uint Voxel_MaxSegmentPerVoxel;
uint Voxel_OffsetAndCount_MaxCount;
uint Voxel_Data_MaxCount;

StructuredBuffer<uint2> Voxel_OffsetAndCount;
StructuredBuffer<uint2> Voxel_Data;
RWTexture2DArray<float4> OutTexture;

Texture2D<float4> PositionTexture;
Texture2D<float4> TangentTexture;
Texture2D<float4> BitangentTexture;
Texture2D<float4> NormalTexture;
Texture2D<float2> UVTexture;

uint3 PositionToCoord(float3 P, float3 InMinAABB, float3 InMaxAABB, uint3 InResolution)
{
	return clamp(
		saturate((P - InMinAABB) / (InMaxAABB - InMinAABB)) * InResolution,
		uint3(0, 0, 0),
		InResolution - 1);
}

uint CoordToIndex(uint3 InCoord, uint3 InResolution, uint LinearOffset)
{
	return
		InCoord.x +
		InCoord.y * InResolution.x +
		InCoord.z * InResolution.x * InResolution.y +
		LinearOffset;
}


bool ClipRaySegment(float3 AABBMin, float3 AABBMax, float4 P0, float4 P1, out float2 T, out bool3 bClipped)
{
	bClipped = false;
	T = float2(0.0f, 1.0f);
	const bool bP0Outside = any(P0.xyz < AABBMin) || any(P0.xyz > AABBMax);
	const bool bP1Outside = any(P1.xyz < AABBMin) || any(P1.xyz > AABBMax);
	if (!bP0Outside && !bP1Outside)
	{
		return true;
	}

	const float3 Origin = P0.xyz;
	const float3 Dir = P1.xyz - P0.xyz;
	const float3 RcpDir = 1.0f / Dir;

	const float3 T0 = (AABBMin - Origin) * RcpDir;
	const float3 T1 = (AABBMax - Origin) * RcpDir;

	T.x = max3(min(T0.x, T1.x), min(T0.y, T1.y), min(T0.z, T1.z));
	T.y = min3(max(T0.x, T1.x), max(T0.y, T1.y), max(T0.z, T1.z));

	// Ray intersects the AABB but the segment is completely outside or no intersection at all.
	if (T.y < 0.0f || T.x > T.y)
	{
		bClipped = true;
		return false;
	}

	if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
	{
		bClipped.x = true;
	}
	if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
	{
		bClipped.y = true;
	}

	return true;
}

bool ClipRaySegment(float3 AABBMin, float3 AABBMax, inout float4 P0, inout float4 P1, inout float Rad0, inout float Rad1, out bool3 bClipped)
{
	float2 T;
	bool bIsValid = ClipRaySegment(AABBMin, AABBMax, P0, P1, T, bClipped);
	
	if (bIsValid)
	{
		const bool bP0Outside = any(P0.xyz < AABBMin) || any(P0.xyz > AABBMax);
		const bool bP1Outside = any(P1.xyz < AABBMin) || any(P1.xyz > AABBMax);

		float4 P0New = P0;
		float4 P1New = P1;
		float Rad0New = Rad0;
		float Rad1New = Rad1;
		if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
		{
			P0New = lerp(P0, P1, T.x);
			Rad0New = lerp(Rad0, Rad1, T.x);
			bClipped.x = true;
		}
		if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
		{
			P1New = lerp(P0, P1, T.y);
			Rad1New = lerp(Rad0, Rad1, T.y);
			bClipped.y = true;
		}

		P0 = P0New;
		P1 = P1New;
		Rad0 = Rad0New;
		Rad1 = Rad1New;
	}

	return bIsValid;
}

bool Trace(
	float3 InRayO,
	float3 InRayD,
	float InMaxDistance,
	float InPixelRadius,
	bool bDebugEnable,
	inout float  OutDepth,
	inout float  OutCoverage,
	inout float3 OutTangent,
	inout float2 OutRootUV, 
	inout float  OutU, 
	inout float  OutSeed,
	inout float4 OutMaterial)
{
	bool bIsValid = false;

	const float3 WP0 = InRayO;
	const float3 WP1 = InRayO + InRayD * InMaxDistance;

	#if DEBUG_ENABLE
	if (bDebugEnable)
	{
		const float4 Color0 = float4(0, 1, 0, 1);
		const float4 Color1 = float4(1, 0, 0, 1);
		AddLineWS(WP0, WP1, Color0, Color1);
	}
	#endif	


	const float2 HitT = LineBoxIntersect(WP0, WP1, Voxel_MinBound, Voxel_MaxBound);
	if (HitT.x < HitT.y)
	{
		#if DEBUG_ENABLE
		if (bDebugEnable)
		{
			//AddLine(true, O, E);
			AddAABBWS(Voxel_MinBound, Voxel_MaxBound, float4(1, 1, 1, 1));
		}
		#endif	

		// Count the number of fibers which are within a cylinder defined by the voxel size, 
		// and the distance between the origin and the extent of the volume
		// This assumes that the voxel volume is cubic (i.e. equal dimensions on all sides)
		const float3 O = WP0; // lerp(WorldPosition, IntersectEndPoint, HitT.xxx);
		const float3 E = WP1; // lerp(WorldPosition, IntersectEndPoint, HitT.yyy);
		const float OELength = min(length(E - O), InMaxDistance);

		const int3 StartCoord = PositionToCoord(WP0, Voxel_MinBound, Voxel_MaxBound, Voxel_Resolution);
		const int3 EndCoord   = PositionToCoord(WP1, Voxel_MinBound, Voxel_MaxBound, Voxel_Resolution);


		// Init to 1 or -1 depending of the orientation of stepping
		const float3 UNormD = WP1 - WP0;
		const int3 Step = sign(UNormD);

		// Step according to voxel size
		const float3 D = normalize(UNormD) * Voxel_Size;

		float t = HitT.x;

		// this is slop coeff for each axis: how far we need to move in units of t for each axis
		const float3 tDelta = Step * Voxel_Size / UNormD;

		// Init to the starting voxel
		int3 PageIndexCoord = -1;
		float3 tMax = 0;
		{
			const float3 HitP = O + HitT.x * UNormD;

			const float Epsilon = 0.000001f;
			const float3 Coords = clamp(
				saturate((HitP - Voxel_MinBound) / (Voxel_MaxBound - Voxel_MinBound)) * Voxel_Resolution,
				0,
				Voxel_Resolution - Epsilon);

			const float3 FractCoords = max(Step, 0) - Step * frac(Coords);
			tMax = FractCoords * tDelta;

			PageIndexCoord = clamp(uint3(Coords), uint3(0, 0, 0), Voxel_Resolution - 1);
		}

		const uint LoopCount = 256u;
		for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
		{
			const bool bIsInside =
				PageIndexCoord.x >= 0 &&
				PageIndexCoord.y >= 0 &&
				PageIndexCoord.z >= 0 &&
				PageIndexCoord.x < int(Voxel_Resolution.x) &&
				PageIndexCoord.y < int(Voxel_Resolution.y) &&
				PageIndexCoord.z < int(Voxel_Resolution.z);
			if (!bIsInside)
			{
				return bIsValid;
			}

			const float3 WPCurr = (PageIndexCoord + 0.5f) * Voxel_Size  + Voxel_MinBound;
			const bool bExit = dot((WPCurr - O), InRayD) > (InMaxDistance + Voxel_Size);
			if (bExit)
			{
				return bIsValid;
			}

			const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, Voxel_Resolution, 0);

			#if DEBUG_ENABLE
			if (bDebugEnable)
			{
				const float3 MinVoxelAABB = Voxel_MinBound + PageIndexCoord * Voxel_Size;
				const float3 MaxVoxelAABB = Voxel_MinBound + (PageIndexCoord + 1) * Voxel_Size;
				AddAABBWS(MinVoxelAABB, MaxVoxelAABB, float4(0, 1, 1, 1));
			}
			#endif	

			// Trace against voxel content
			{
				const int3 VolumeCoord = PageIndexCoord;

				// Update page index only when needed
				uint2 OffsetAndCount = 0;
				{
					const uint LinearIndexCoord = CoordToIndex(VolumeCoord, Voxel_Resolution, 0);
					OffsetAndCount = Voxel_OffsetAndCount[clamp(LinearIndexCoord, 0, Voxel_OffsetAndCount_MaxCount-1)];
					OffsetAndCount.y = min(OffsetAndCount.y, Voxel_MaxSegmentPerVoxel);
				}

				if (OffsetAndCount.y > 0)
				{
					for (uint PointIt = 0; PointIt < OffsetAndCount.y; ++PointIt)
					{
						const uint VoxelIndex = clamp(OffsetAndCount.x + PointIt, 0, Voxel_Data_MaxCount-1);
						const uint2 P0_P1 = Voxel_Data[VoxelIndex].xx + uint2(0,1);

						const uint P0 = P0_P1.x;
						const uint P1 = P0_P1.y;

						FHairControlPoint CP_Curr = ReadHairControlPoint(InVF_PositionBuffer, P0, InVF_PositionOffset, InVF_Radius, InVF_RootScale, InVF_TipScale);
						FHairControlPoint CP_Next = ReadHairControlPoint(InVF_PositionBuffer, P1, InVF_PositionOffset, InVF_Radius, InVF_RootScale, InVF_TipScale);

						// Clip segment to voxel boundary in order to avoid counting multiple time the same segment which would cause the coverage to be incorrect 
						// as the ray trace several voxels
						{
							const float3 MinVoxelAABB = Voxel_MinBound + PageIndexCoord * Voxel_Size;
							const float3 MaxVoxelAABB = Voxel_MinBound + (PageIndexCoord + 1) * Voxel_Size;
						
							float4 ClippedP0  = float4(CP_Curr.Position, 1);
							float4 ClippedP1  = float4(CP_Next.Position, 1);
							float ClippedRad0 = CP_Curr.WorldRadius;
							float ClippedRad1 = CP_Next.WorldRadius;
							bool3 bClipped    = false;

							ClipRaySegment(MinVoxelAABB, MaxVoxelAABB, ClippedP0, ClippedP1, ClippedRad0, ClippedRad1, bClipped);

							if (any(bClipped))
							{
								CP_Curr.Position    = ClippedP0.xyz;
								CP_Next.Position    = ClippedP1.xyz;
								CP_Curr.WorldRadius = ClippedRad0;
								CP_Next.WorldRadius = ClippedRad1;
							}
						}

						const float3 Tangent = ComputeTangent(CP_Curr, CP_Next);

						const float3 RayP0 = InRayO;
						const float3 RayP1 = InRayO + InRayD * InMaxDistance;

						float VertexU = 0;
						const float WorldRadius = 2 * max(CP_Curr.WorldRadius, CP_Next.WorldRadius); //  0.05f;
						const float Distance = Intersection(CP_Curr.Position, CP_Next.Position, RayP0, RayP1, WorldRadius, VertexU) * InMaxDistance;

						const bool bIntersect = Distance > 0;
						const bool bClosest = Distance < OutDepth;

						const bool bIntersectAndClosest = bIntersect && bClosest;

						if (bIntersect)
						{
							const float SegmentCoverage = saturate(WorldRadius / InPixelRadius);
							//OutCoverage = saturate(OutCoverage + saturate(1.f-OutCoverage) * SegmentCoverage);
							OutCoverage = saturate(OutCoverage + SegmentCoverage);
							//OutCoverage = 1;
						}

						if (bIntersectAndClosest)
						{
							OutTangent 	= Tangent;
							OutDepth 	= Distance;
							OutSeed		= GetHairStrandsSeed(P0);
							OutRootUV	= GetHairStrandsRootUV(P0);
							OutU		= GetHairStrandsUV(P0, VertexU).x; 
							OutMaterial	= 0;

							if (HasHairStrandsColor())
							{
								OutMaterial.xyz = GetHairStrandsColor(P0, VertexU);
							}
							if (HasHairStrandsRoughness())
							{
								OutMaterial.w = GetHairStrandsRoughness(P0, VertexU);
							}

							bIsValid = true;

							#if DEBUG_ENABLE
							if (bDebugEnable)
							{
								const float4 Color = float4(1, 0, 0, 1);
								AddLineWS(CP_Curr.Position, CP_Next.Position, Color, Color);
								AddCrossWS(CP_Curr.Position, 0.05f, Color);
								AddCrossWS(CP_Next.Position, 0.05f, Color);
							}
							#endif
						}
					}
				}
			}

			// t is used for defining the intersection point at the entry of a valid page
			t = min(tMax.x, min(tMax.y, tMax.z));

			// Find the next page indx to visit and update the tmax, accordingly
			const float3 Mask = tMax.x < tMax.y ?
				(tMax.x < tMax.z ? float3(1, 0, 0) : float3(0, 0, 1)) :
				(tMax.y < tMax.z ? float3(0, 1, 0) : float3(0, 0, 1));
			PageIndexCoord += Step * Mask;
			tMax += tDelta * Mask;
		}
	}

	return bIsValid;
}

[numthreads(8, 8, 1)]
void MainCS(
	uint GroupIndex : SV_GroupIndex, 
	uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const int2 PixelCoord = DispatchThreadId.xy + TileOffsetInPixels;
	if (any(PixelCoord > OutputResolution))
	{
		return;
	}

	const float2 InUV		= UVTexture[PixelCoord];
	const float3 InPosition = PositionTexture[PixelCoord].xyz;
	const float3 InTangent	= TangentTexture[PixelCoord].xyz;
	const float3 InBitangent= BitangentTexture[PixelCoord].xyz;
	const float3 InNormal	= NormalTexture[PixelCoord].xyz;

	const float3 T = normalize(InTangent);
	const float3 B = normalize(InBitangent);
	const float3 N = normalize(InNormal);

	// Manual ddx/ddy
	float3 PixelAxisX = 0;
	float3 PixelAxisY = 0; 
	if (PixelCoord.x + 1 < OutputResolution.x)
	{
		PixelAxisX = PositionTexture[float2(PixelCoord.x+1, PixelCoord.y)].xyz - InPosition;
	}
	if (PixelCoord.y + 1 < OutputResolution.y)
	{
		PixelAxisY = PositionTexture[float2(PixelCoord.x, PixelCoord.y+1)].xyz - InPosition;
	}
	const float PixelWorldRadius = 0.25f * (length(PixelAxisY) + length(PixelAxisX)); // Average + pixel radius (vs. diameter)

	float4 OutTexture0 = 0;
	float4 OutTexture1	= 0;
	float4 OutTexture2	= 0;
	float4 OutTexture3	= 0;
	float4 OutTexture4	= 0;
	float4 OutTexture5	= 0;

	float  OutDepth = max(0, MaxDistance);
	float  OutSeed = 0;
	float  OutCoordU = 0;
	float  OutGroupIndex = 0;
	float  OutRoughness = 0;
	float2 OutRootUV = 0;
	float3 OutColor = 0;
	float3 OutTangent = 0;
	float  OutCoverage = 0;

	float3 Tangent_ObjectSpace = 0;
	const bool bDebugEnable =
	#if DEBUG_ENABLE
		all(PixelCoord == GetCursorPos());
	#else
		false;
	#endif

	uint bValidCount = 0;

	const uint SampleCount = 16;
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float2 Jitter = 0.5f * (Hammersley(SampleIt, SampleCount, 0) * 2 - 1);

		float3 RayO = InPosition + Jitter.x * PixelAxisX + Jitter.y * PixelAxisY;
		float3 RayD = N;

		// If the tracing distance is:
		// * Positive: Tracing is done outside towards the surface
		// * Negative: Tracing is done inside, starting from the surface
		// * Zero    : Tracing is from outside towards the surface, but continus within the surface
		if (TracingDirection > 0)
		{
			RayO = RayO + RayD * MaxDistance;
			RayD =-RayD;
		}
		else if (TracingDirection < 0)
		{
			RayD = -RayD;
		}
		else // (TracingDirection == 0)
		{
			RayO = RayO + RayD * MaxDistance * 0.5f;
			RayD = -RayD;
		}
	
		float  SampleDepth = MaxDistance;
		float  SampleCoverage = 0;
		float3 SampleTangent  = 0;
		float2 SampleRootUV = 0;
		float  SampleU = 0;
		float  SampleSeed = 0;
		float4 SampleMaterial = 0;
		if (Trace(RayO, RayD, MaxDistance, PixelWorldRadius, bDebugEnable, SampleDepth, SampleCoverage, SampleTangent, SampleRootUV, SampleU, SampleSeed, SampleMaterial))
		{
			if (SampleDepth < OutDepth)
			{
				OutRootUV    = SampleRootUV;
				OutCoordU    = SampleU;
				OutSeed      = SampleSeed;
				OutColor     = SampleMaterial.xyz;
				OutRoughness = SampleMaterial.w;
			}

			OutDepth = min(OutDepth, SampleDepth);
			OutCoverage += SampleCoverage;
			Tangent_ObjectSpace += SampleTangent;
			++bValidCount;
		}
	}

	if (bValidCount > 0 && OutCoverage > 0)
	{
		Tangent_ObjectSpace = normalize(Tangent_ObjectSpace);
		const float3 Tangent_FrameSpace  = float3(
			dot(Tangent_ObjectSpace, T), 
			dot(Tangent_ObjectSpace, B), 
			dot(Tangent_ObjectSpace, N));

		OutCoverage /= SampleCount;
		OutTangent = (Tangent_FrameSpace + 1) * 0.5f;
		OutGroupIndex = InVF_GroupIndex / 255.f;
	}
	else
	{
		OutDepth = MaxDistance;
		OutSeed = 0;
		OutCoordU = 0;
		OutGroupIndex = 0;
		OutRoughness = 0;
		OutRootUV = 0;
		OutColor = 0;
		OutTangent = 0;
		OutCoverage = 0;
	}

	OutDepth = saturate(OutDepth / MaxDistance);

	if (bValidCount > 0)
	{
		WriteValue3(LayoutIndex, EHairCardsVFAttribute_Color,      OutColor);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Depth,      OutDepth);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Seed,       OutSeed);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_CoordU,     OutCoordU);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_GroupIndex, OutGroupIndex);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Roughness,  OutRoughness);
		WriteValue2(LayoutIndex, EHairCardsVFAttribute_RootUV,     OutRootUV);
		WriteValue3(LayoutIndex, EHairCardsVFAttribute_Tangent,    OutTangent);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Coverage,   OutCoverage);

		OutTexture[uint3(PixelCoord, 0)] = OutTexture0;
		OutTexture[uint3(PixelCoord, 1)] = OutTexture1;
		OutTexture[uint3(PixelCoord, 2)] = OutTexture2;
		OutTexture[uint3(PixelCoord, 3)] = OutTexture3;
		OutTexture[uint3(PixelCoord, 4)] = OutTexture4;
		OutTexture[uint3(PixelCoord, 5)] = OutTexture5;
	}
}
#endif // SHADER_COMPUTE

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TEXTURE_DILATION

#include "HairCardsAttributeCommon.ush"

uint LayoutIndex;
uint bClearPass;
uint TextureCount;
int2 Resolution;
int ClearDistance;
Texture2D<float> DepthTestTexture;
Texture2D<uint> OriginalTriangleMaskTexture;
Texture2D<uint> SourceTriangleMaskTexture;
RWTexture2D<uint> TargetTriangleMaskTexture;
Texture2DArray<float4> SourceTexture;
RWTexture2DArray<float4> TargetTexture;

// Dilate the source texture (Depth/Coverage/Tanget/Attribute), using a distance to triangle edge metric. 
// Original valid pixel are marked into the triangleMasktexture during the texture generation. Then 
// successive dilation pass are run. At each pass the triangleMaskTexure stored the distance to triangle
// edge in order to propagate correctly the pixel edge information.
[numthreads(8, 8, 1)]
void MainCS(int2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord >= uint2(Resolution)))
	{
		return;
	}

	bool bIsValid = false;

	// Pixel within the source, which are not written due to lack of data, will be dialated as well
	{
		const uint CoverageTextureIndex = GetHairCardTextureIndex(LayoutIndex, EHairCardsVFAttribute_Coverage);
		const uint CoverageChannelIndex = GetHairCardChannelIndex(LayoutIndex, EHairCardsVFAttribute_Coverage);
		const float4 Value = SourceTexture[uint3(PixelCoord, CoverageTextureIndex)];
		float CoverageValue = 1.f;
		switch (CoverageChannelIndex)
		{
			case 0 : CoverageValue = Value.x; break;
			case 1 : CoverageValue = Value.y; break;
			case 2 : CoverageValue = Value.z; break;
			case 3 : CoverageValue = Value.z; break;
		}

		// * When doing the clear pass, only mark as valid, pixels whose coverage is > 0
		// * For other passes, use the source mask (i.e, the mask updated at every pass) for identifying valid/already processed pixels
		if (bClearPass)
		{
			bIsValid = CoverageValue > 0.f;
		}
		else
		{
			bIsValid = SourceTriangleMaskTexture.Load(uint3(PixelCoord,0)) > 0;
		}
	}

	if (bClearPass || bIsValid)
	{
		// Fast path: the pixel is valid, we copy value
		for (uint TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
		{
			TargetTexture[uint3(PixelCoord, TextureIt)] = SourceTexture[uint3(PixelCoord, TextureIt)];
		}

		if (bClearPass)
		{
			TargetTriangleMaskTexture[PixelCoord] = (bIsValid && OriginalTriangleMaskTexture.Load(uint3(PixelCoord,0)) > 0.5f) ? 1u : 0u;
		}
		else
		{
			TargetTriangleMaskTexture[PixelCoord] = SourceTriangleMaskTexture.Load(uint3(PixelCoord,0));
		}
	}
	else
	{
		const uint CoverageTextureIndex = GetHairCardTextureIndex(LayoutIndex, EHairCardsVFAttribute_Coverage);
		const uint CoverageChannelIndex = GetHairCardChannelIndex(LayoutIndex, EHairCardsVFAttribute_Coverage);

		// Find the closest neighbor from triangle, amoung valid neighbord
		float4 Values[HAIR_CARDS_MAX_TEXTURE_COUNT];
		for (uint TextureIt = 0; TextureIt < HAIR_CARDS_MAX_TEXTURE_COUNT; ++TextureIt)
		{
			Values[TextureIt] = 0;
		}
		bool bIsDilatedPixelValid = false;
		float MinDistance = 9999999;
		for (int y = -1; y <= 1; ++y)
		for (int x = -1; x <= 1; ++x)
		{
			if (x == 0 && y == 0)
				continue;

			const uint2 NPixelCoord = clamp(int2(PixelCoord) + int2(x, y), int2(0,0), int2(Resolution-1));

			const bool bNIsValid = SourceTriangleMaskTexture.Load(uint3(NPixelCoord,0)) > 0;
			if (bNIsValid)
			{
				for (uint TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
				{
					Values[TextureIt] = SourceTexture[uint3(NPixelCoord, TextureIt)];
				}
				bIsDilatedPixelValid= true;
			}
		}

		// For coverage dialation, we only allow dilation outside of the triangles
		const bool bIsInsideTriangle = OriginalTriangleMaskTexture.Load(uint3(PixelCoord,0)) > 0;

		// If a valid neighbord has been bound, update pixel value
		if (bIsDilatedPixelValid)
		{
			for (uint TextureIt = 0; TextureIt < TextureCount; ++TextureIt)
			{
				// Write all attributes but coverage, as coverage should remain untouched.
				float4 WriteOut = Values[TextureIt];
				if (bIsInsideTriangle && TextureIt == CoverageTextureIndex)
				{
					switch (CoverageChannelIndex)
					{
						case 0 : WriteOut.x = 0.f; break;
						case 1 : WriteOut.y = 0.f; break;
						case 2 : WriteOut.z = 0.f; break;
						case 3 : WriteOut.w = 0.f; break;
					}
				}
				TargetTexture[uint3(PixelCoord, TextureIt)] = WriteOut;
			}
			TargetTriangleMaskTexture[PixelCoord] = 1.f;
		}
	}
}
#endif // SHADER_TEXTURE_DILATION

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_READ_POSITIONS

#include "HairStrandsVertexFactoryCommon.ush"

uint MaxVertexCount;
ByteAddressBuffer InPositions;
Buffer<float4> InPositionOffset;
RWBuffer<float4> OutPositions;
[numthreads(128, 1, 1)]
void MainCS(int2 DispatchThreadId : SV_DispatchThreadID)
{
	uint VertexIndex = DispatchThreadId.x;
	if (VertexIndex < MaxVertexCount)
	{
		const float3 PositionOffset = InPositionOffset[0].xyz;
		const FHairControlPoint CP = ReadHairControlPoint(InPositions, VertexIndex, PositionOffset, 1.0f /*InVF_Radius*/, 1.0f /*InVF_RootScale*/, 1.0f /*InVF_TipScale*/);
		
		OutPositions[VertexIndex] = float4(CP.Position, CP.Type);
	}
}
#endif // SHADER_READ_POSITIONS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TEXTURE_CLEAR

#include "HairCardsAttributeCommon.ush"
int2 Resolution;
uint LayoutIndex;
RWTexture2DArray<float4> OutTexture;

[numthreads(8, 8, 1)]
void MainCS(int2 DispatchThreadId : SV_DispatchThreadID)
{
	int2 PixelCoord = DispatchThreadId.xy;
	if (all(PixelCoord < Resolution))
	{	
		float4 OutTexture0 = 0;
		float4 OutTexture1 = 0;
		float4 OutTexture2 = 0;
		float4 OutTexture3 = 0;
		float4 OutTexture4 = 0;
		float4 OutTexture5 = 0;

		WriteValue3(LayoutIndex, EHairCardsVFAttribute_Color,      0.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Depth,      1.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Seed,       0.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_CoordU,     0.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_GroupIndex, 0.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Roughness,  0.f);
		WriteValue2(LayoutIndex, EHairCardsVFAttribute_RootUV,     0.f);
		WriteValue3(LayoutIndex, EHairCardsVFAttribute_Tangent,    0.f);
		WriteValue1(LayoutIndex, EHairCardsVFAttribute_Coverage,   0.f);

		OutTexture[uint3(PixelCoord, 0)] = OutTexture0;
		OutTexture[uint3(PixelCoord, 1)] = OutTexture1;
		OutTexture[uint3(PixelCoord, 2)] = OutTexture2;
		OutTexture[uint3(PixelCoord, 3)] = OutTexture3;
		OutTexture[uint3(PixelCoord, 4)] = OutTexture4;
		OutTexture[uint3(PixelCoord, 5)] = OutTexture5;
	}
}
#endif // SHADER_TEXTURE_CLEAR

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TEXTURE_COPY

uint TextureIndex;
int2 Resolution;
Texture2DArray<float4> InTexture;
RWTexture2D<float4> OutTexture;

[numthreads(8, 8, 1)]
void MainCS(int2 DispatchThreadId : SV_DispatchThreadID)
{
	int2 PixelCoord = DispatchThreadId.xy;
	if (all(PixelCoord < Resolution))
	{	
		OutTexture[PixelCoord] = InTexture[uint3(PixelCoord, TextureIndex)];
	}
}
#endif // SHADER_TEXTURE_COPY