// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Default tile size
#define HAIR_TILE_SIZE 8

// Tile types
#define HAIRTILE_HAIR_ALL 0 
#define HAIRTILE_HAIR_FULL 1 
#define HAIRTILE_HAIR_PARTIAL 2
#define HAIRTILE_OTHER 3
#define HAIRTILE_COUNT 4