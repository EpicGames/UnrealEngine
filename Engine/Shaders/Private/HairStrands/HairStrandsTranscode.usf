// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"
#include "../PackUnpack.ush"
#include "HairStrandsPack.ush"

#if SHADER_TRANSCODE

uint OffsetInBytes;
uint DataSizeInBytes;
uint TotalSizeInBytes;

uint UncompressedOffsetInBytes;
uint UncompressedTotalSizeInBytes;

float3 PositionScale;
float3 PositionOffset;

ByteAddressBuffer InBuffer;
RWByteAddressBuffer OutBuffer;

#if GROUP_SIZE != 1024
#error Group size is not == 1024u. Please update this shader.
#endif

float UnpackR7(uint In) { return In / 127.f; }
float UnpackR2(uint In) { return In / 3.f; }

FHairControlPoint UnpackTranscoodedPosition(uint In)
{
	FHairControlPoint Out;
	Out.Position.x = UnpackUnorm10(BitFieldExtractU32(In, 10,  0));
	Out.Position.y = UnpackUnorm10(BitFieldExtractU32(In, 10, 10));
	Out.Position.z = UnpackUnorm10(BitFieldExtractU32(In, 10, 20));
	Out.Type       = BitFieldExtractU32(In, 2, 30);

	Out.Position   = Out.Position * PositionScale + PositionOffset;
	return Out;
}

void UnpackUCoord(inout FHairControlPoint InCP, uint DataW0, uint DataW1, inout uint bFirst)
{
	switch(InCP.Type)
	{
		case HAIR_CONTROLPOINT_START: 	InCP.UCoord = 0u; break;
		case HAIR_CONTROLPOINT_END:		InCP.UCoord = 1u; break;
		case HAIR_CONTROLPOINT_INSIDE:	InCP.UCoord = bFirst ? DataW0 : DataW1; bFirst = false; break;
	}
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint ThreadIndex : SV_DispatchThreadID)
#if PERMUTATION_TRANSCODING
{
	// 3 CPs per 128bits/16bytes
	const uint InAddressInBytes  = ThreadIndex * FCompressedHairPositionsStrideInBytes;

	if (InAddressInBytes + FCompressedHairPositionsStrideInBytes <= DataSizeInBytes)
	{
		const uint4 Data = InBuffer.Load4(InAddressInBytes);
	
		// Sanity check
		#if HAIR_POINT_COUNT_PER_COMPRESSED_POSITION_CHUNK != 3
		#error Update hair point decompression code. Assume 3 CPs per compressed chunk
		#endif

		// Position - 3x32bits
		// Each position - 10|10|10|2 -> 3x16bits
		FHairControlPoint CP0 = UnpackTranscoodedPosition(Data.x);
		FHairControlPoint CP1 = UnpackTranscoodedPosition(Data.y);
		FHairControlPoint CP2 = UnpackTranscoodedPosition(Data.z);

		// Radius - 18bits
		{
			CP0.WorldRadius = UnpackR6(BitFieldExtractU32(Data.w, 6, 0));
			CP1.WorldRadius = UnpackR6(BitFieldExtractU32(Data.w, 6, 6));
			CP2.WorldRadius = UnpackR6(BitFieldExtractU32(Data.w, 6, 12));
		}

		// CoordU - 14bits
		if (CP0.Type==HAIR_CONTROLPOINT_INSIDE && 
			CP1.Type==HAIR_CONTROLPOINT_INSIDE && 
			CP2.Type==HAIR_CONTROLPOINT_INSIDE)
		{
			// The 3 CPs are consecutive and we compress them by storing begin/end points + lerping the middle point
			CP0.UCoord = UnpackR6(BitFieldExtractU32(Data.w, 6, 18));
			CP2.UCoord = UnpackR6(BitFieldExtractU32(Data.w, 6, 24));

			const float S = UnpackR2(BitFieldExtractU32(Data.w, 2, 30));
			CP1.UCoord = lerp(CP0.UCoord, CP2.UCoord, S);
		}
		else
		{
			// In this configuration 1 or 2 CPs are begin or end of a curve, meaning that their CoordU value is either 0 or 1
			// We use the 14bit to store 2x7bits radius information
			const uint DataW0 = UnpackR7(BitFieldExtractU32(Data.w, 7, 18));
			const uint DataW1 = UnpackR7(BitFieldExtractU32(Data.w, 7, 25));

			bool bFirst = true;
			UnpackUCoord(CP0, DataW0, DataW1, bFirst);
			UnpackUCoord(CP1, DataW0, DataW1, bFirst);
			UnpackUCoord(CP2, DataW0, DataW1, bFirst);
		}

		const uint OutAddress0 = UncompressedOffsetInBytes + (ThreadIndex*3 + 0) * FPackedHairPositionStrideInBytes;
		const uint OutAddress1 = UncompressedOffsetInBytes + (ThreadIndex*3 + 1) * FPackedHairPositionStrideInBytes;
		const uint OutAddress2 = UncompressedOffsetInBytes + (ThreadIndex*3 + 2) * FPackedHairPositionStrideInBytes;

		if (OutAddress0+FPackedHairPositionStrideInBytes <= UncompressedTotalSizeInBytes) { OutBuffer.Store2(OutAddress0, PackHairControlPoint(CP0, 0 /*PositionOffset*/, 1 /*Radius*/)); }
		if (OutAddress1+FPackedHairPositionStrideInBytes <= UncompressedTotalSizeInBytes) { OutBuffer.Store2(OutAddress1, PackHairControlPoint(CP1, 0 /*PositionOffset*/, 1 /*Radius*/)); }
		if (OutAddress2+FPackedHairPositionStrideInBytes <= UncompressedTotalSizeInBytes) { OutBuffer.Store2(OutAddress2, PackHairControlPoint(CP2, 0 /*PositionOffset*/, 1 /*Radius*/)); }
	}
}
#else //PERMUTATION_TRANSCODING
{
	// Simple copy	
	const uint InAddressInBytes  = ThreadIndex * 4u;
	const uint OutAddressInBytes = OffsetInBytes + ThreadIndex * 4u;

	if (InAddressInBytes < DataSizeInBytes && OutAddressInBytes < TotalSizeInBytes)
	{
		const uint InData = InBuffer.Load(InAddressInBytes);
		OutBuffer.Store(OutAddressInBytes,InData);
	}
}
#endif // PERMUTATION_TRANSCODING
#endif // SHADER_TRANSCODE