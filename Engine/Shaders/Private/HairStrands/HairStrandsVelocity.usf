// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsTileCommon.ush"

#define TILE_PIXEL_SIZE_X 8
#define TILE_PIXEL_SIZE_Y 8

#if (TILE_PIXEL_SIZE_X != HAIR_TILE_SIZE) || (TILE_PIXEL_SIZE_Y != HAIR_TILE_SIZE) 
#error Tile size needs to match 
#endif

int2 Resolution;
int2 ResolutionOffset;
float VelocityThreshold;
float CoverageThreshold;
uint bNeedClear;

Texture2D<uint> NodeIndex;
Buffer<float4> NodeVelocity;
StructuredBuffer<FPackedHairVis> NodeVis; 
Texture2D<float> CoverageTexture;

#define VELOCITY_TYPE_NONE 0
#define VELOCITY_TYPE_AVG 1  
#define VELOCITY_TYPE_CLOSEST 2	
#define VELOCITY_TYPE_MAX 3
#if PERMUTATION_VELOCITY != VELOCITY_TYPE_NONE
  #if PERMUTATION_OUTPUT_FORMAT == 0
	RWTexture2D<float2> OutVelocityTexture;
  #else
	RWTexture2D<float4> OutVelocityTexture;
  #endif
#endif

uint TileSize;
int2 TileCountXY;
Buffer<uint> TileCountBuffer;
Buffer<uint2> TileDataBuffer;

RWTexture2D<uint> OutResolveMaskTexture;

#if PERMUTATION_OUTPUT_FORMAT == 0
float2 ToOutput(float4 In) { return In.xy; }
#else
float4 ToOutput(float4 In) { return In; }
#endif

[numthreads(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint2 InGroupId : SV_GroupID, uint2 InGroupThreadId : SV_GroupThreadID)
{
	const uint TileCount = TileCountBuffer[HAIRTILE_HAIR_ALL];
	const uint TileIndex1D = InGroupId.x + InGroupId.y * TileCountXY.x;
	if (TileIndex1D >= TileCount)
	{
		return;
	}
	const uint2 GroupId = TileDataBuffer[TileIndex1D];
	const uint2 PixelCoord = uint2(View.ViewRectMin.xy) + GroupId * TileSize + InGroupThreadId;

	const FNodeDesc NodeDesc = DecodeNodeDesc(NodeIndex.Load(uint3(PixelCoord, 0)));
	bool bNeedFastResolve = false;
	bool bValidVelocity = false;
	float4 OutEncodedVelocity = 0;
	if (NodeDesc.Count > 0)
	{
		// Store final sort node data 
		#if PERMUTATION_VELOCITY == VELOCITY_TYPE_AVG
		float3 AverageVelocity = 0;
		#endif

		#if PERMUTATION_VELOCITY == VELOCITY_TYPE_CLOSEST
		float4 ClosestEncodedVelocity = 0;
		float ClosestDepth = 0; // Inverse-Z
		#endif		

		#if PERMUTATION_VELOCITY == VELOCITY_TYPE_MAX
		float4 MaxEncodedVelocity = 0;
		float MaxVelocityMagnitude2 = 0;
		#endif		

		const float HairPixelCoverage = CoverageTexture.Load(uint3(PixelCoord,0));
		if (HairPixelCoverage >= CoverageThreshold)
		{
			bValidVelocity = true;
			for (uint NodeIt = 0; NodeIt < NodeDesc.Count; ++NodeIt)
			{
				const uint SampleIndex = NodeDesc.Offset + NodeIt;
				const float4 EncodedVelocity = NodeVelocity[SampleIndex];

				#if PERMUTATION_VELOCITY == VELOCITY_TYPE_AVG
				AverageVelocity += DecodeVelocityFromTexture(EncodedVelocity);
				#endif			

				#if PERMUTATION_VELOCITY == VELOCITY_TYPE_CLOSEST
				const float NodeDepth = UnpackHairVis(NodeVis[SampleIndex]).Depth;
				if (NodeDepth > ClosestDepth) // Inverse-Z
				{
					ClosestEncodedVelocity = EncodedVelocity;
					ClosestDepth = NodeDepth;
				}
				#endif

				#if PERMUTATION_VELOCITY == VELOCITY_TYPE_MAX
				const float3 CurrentVelocity = DecodeVelocityFromTexture(EncodedVelocity);
				const float CurrentVelocityMagnitude2 = dot(CurrentVelocity, CurrentVelocity);
				if (CurrentVelocityMagnitude2 > MaxVelocityMagnitude2)
				{
					MaxEncodedVelocity = EncodedVelocity;
					MaxVelocityMagnitude2 = CurrentVelocityMagnitude2;
				}
				#endif			

				// If the velocity is above a certain threshold, the pixel will be resolve with a fast resolved. 
				// This will result into a sharper, but more noisy output. However it sill avoid getting smearing 
				// from TAA.
				bNeedFastResolve = bNeedFastResolve || NeedFastResolve(EncodedVelocity.xy, VelocityThreshold);
			}

			#if PERMUTATION_VELOCITY == VELOCITY_TYPE_AVG
			OutEncodedVelocity = EncodeVelocityToTexture(float3(AverageVelocity / max(NodeDesc.Count, 1u)));
			#endif		

			#if PERMUTATION_VELOCITY == VELOCITY_TYPE_CLOSEST
			OutEncodedVelocity = ClosestEncodedVelocity;
			#endif		

			#if PERMUTATION_VELOCITY == VELOCITY_TYPE_MAX
			OutEncodedVelocity = MaxEncodedVelocity;
			#endif
		}
	}

	#if PERMUTATION_VELOCITY != VELOCITY_TYPE_NONE
	if (bValidVelocity || bNeedClear)
	{
		OutVelocityTexture[PixelCoord]  = bValidVelocity ? ToOutput(OutEncodedVelocity) : 0.f;
	}
	#endif
	OutResolveMaskTexture[PixelCoord] = bNeedFastResolve ? 1 : 0;
}
