// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StrandHairFactory.usf
=============================================================================*/
#include "../VertexFactoryCommon.ush"  
#include "../LocalVertexFactoryCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsAttributeCommon.ush"

#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"

#define USE_HAIR_COMPLEX_TRANSMITTANCE 1 

#ifndef USE_HAIR_TRIANGLE_STRIP
	#error Hair triangle geometry type needs to be defined
#endif

#define ENABLE_HAIRSTRANDS_TRIANGLE_INTERSECTOR !(ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR || ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR)

#if MANUAL_VERTEX_FETCH
	Buffer<float4> VertexFetch_InstanceOriginBuffer;
	Buffer<float4> VertexFetch_InstanceTransformBuffer;
#endif

// Enable cluster culling by default as the output of the strands interpolation output culled data, 
// and this allows to support other rendered (e.g., hit proxy, shadow rendering), for which it is not 
// possible to provide custom defines
#ifndef USE_CULLED_CLUSTER
#define USE_CULLED_CLUSTER 1
#endif

////////////////////////////////////////////////////////////////////////////////
// HAIR_STRAND_MESH_FACTORY
// Define used in certain shader like DeepShadow for running specific code (coverage computation for instance) 
// when vertices are produced by the strand hair vertex factory 
// This is set by compilation environment of the vertex factory
///////////////////////////////////////////////////////////////////////////////


// ## A quad is made of the following topology and indexing:
//
//  0__2    4 
//  | /    /|
//  |/    /_|
//  1    5  3  
//
//  ## Control point identification
//  Here is an example of two consecutive strands. The control point:
//   * .a=1 starting
//   * .a=0 within
//   * .a=2 ending
//
//   _O       .a == 1  Strand 0
//  | / /|
//  |/ /_|
//   _O       .a == 0
//  | / /|
//  |/ /_|
//   _O       .a == 0
//  | / /|
//  |/ /_|
//    O       .a == 2
//     
//    
//    O       .a == 1   Strand 1
//  | / /|
//  |/ /_|
//    O       .a == 2
//
//   ...

/**
 * Per-vertex inputs from bound vertex buffers
 */
bool UseStableRasterization()
{
	return HasHairFlags(HairStrandsVF.Flags, HAIR_FLAGS_STABLE_RASTER);
}

bool UseScatterSceneLighting()
{
	return HasHairFlags(HairStrandsVF.Flags,HAIR_FLAGS_SCATTER_SCENE_LIGHT);
}

struct FVertexFactoryInput
{
#if RAYHITGROUPSHADER
	float4 Position;
	uint PrimitiveId;          // from RayHit (either TriangleId or SegmendId)

#if ENABLE_HAIRSTRANDS_TRIANGLE_INTERSECTOR
	uint PrimitiveVertexId;    // 0,1,2 (Vertex within the triangle)
#endif
#endif

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE13
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)

	// Stereo rendering related params
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()

	uint VertexId : SV_VertexID;
};

struct FVertexInfo
{
	uint VertexIndex;
	uint HairControlPointId;
	uint IsTip;				// The vertex is on the quad side toward the tip of the strand.
#if !RAYHITGROUPSHADER
	uint IsLeft;
	bool bForceInvalidQuad;
#endif
};
FVertexInfo GetVertexInfo(FVertexFactoryInput Input)
{
	uint VertexId = Input.VertexId;

	FVertexInfo VertexInfo;
#if RAYHITGROUPSHADER

#if ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR
	VertexInfo.IsTip		= 0;
	uint BaseIndex			= VertexId;
#elif ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR
	VertexInfo.IsTip		= 0;
	uint BaseIndex			= VertexId;
#else
	// Triangle intersection
	uint BaseIndex			= VertexId / 4;
	// TODO: is there any easy way we could handle end-caps?
	VertexInfo.IsTip		= (Input.PrimitiveId % 2) == 0 ? Input.PrimitiveVertexId != 2 : Input.PrimitiveVertexId == 0;
#endif

#else

#if USE_HAIR_TRIANGLE_STRIP
	uint QuadIndex 			= VertexId % 2;
	uint BaseIndex			= VertexId / 2;
	VertexInfo.IsTip		= 0;
	VertexInfo.IsLeft		= QuadIndex == 0 ? 1 : 0;
#else
	uint QuadIndex			= VertexId % 6;
	uint BaseIndex			= VertexId / 6;
	VertexInfo.IsTip		= QuadIndex == 0 || QuadIndex == 2 || QuadIndex == 4 ? 0 : 1;
	VertexInfo.IsLeft		= QuadIndex == 0 || QuadIndex == 1 || QuadIndex == 5 ? 1 : 0;
#endif

#endif
	VertexInfo.HairControlPointId = BaseIndex;
	VertexInfo.VertexIndex = BaseIndex + VertexInfo.IsTip;
#if !RAYHITGROUPSHADER
	VertexInfo.bForceInvalidQuad = false;
#endif

#if USE_CULLED_CLUSTER == 1
	if (HairStrandsVF.bCullingEnable)
	{
		VertexInfo.VertexIndex = HairStrandsVF.CullingIndexBuffer[BaseIndex + VertexInfo.IsTip];
		VertexInfo.HairControlPointId = VertexInfo.VertexIndex - VertexInfo.IsTip;
	}
#endif

	return VertexInfo;
}

float3 RotateLocalHairTangent(float3 In)
{
	// Convert the tangent space input tangent 
	// * from a frame with Z/(0,0,1) being up and Y/(0,1,0) being the main tangent direction (called green)
	// * to   a frame with Y/(0,1,0) being up and Z/(0,0,1) being the main tangent direction (called blue)
	return float3(-In.x, In.z, In.y);
}

// Unlike AssembleTangentToWorld, this helper first adjusts the TangentToWorld0 direction
// using TangentToWorld2 to ensure they are perpendicular before constructing the matrix.
half3x3 AssembleTangentToWorldForHairStrand(half3 TangentToWorld0, half4 TangentToWorld2)
{
	// Derive the third basis vector off of the other two.
	// Flip based on the determinant sign
	const half3 TangentToWorld1 = normalize(cross(TangentToWorld2.xyz,TangentToWorld0) * TangentToWorld2.w);

	// Correct TangentToWorld0 direct in case it is not perpendicular.
	const half3 CorrectedTangentToWorld0 = normalize(cross(TangentToWorld1, TangentToWorld2.xyz));

	// Transform from tangent space to world space
	return half3x3(CorrectedTangentToWorld0, TangentToWorld1, TangentToWorld2.xyz);
}

#if RAYHITGROUPSHADER
FHairControlPoint GetVertexPosition(const FVertexInfo VertexInfo, bool bInvalidJointVertex);

FVertexFactoryInput LoadVertexFactoryInputForHGS(uint PrimitiveIndex, int VertexIndex)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

#if ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR
	if (VertexIndex == 0 || VertexIndex == 1)
	{
		Input.VertexId = PrimitiveIndex + VertexIndex;
		Input.PrimitiveId = PrimitiveIndex;
	}
#elif ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR
	if (VertexIndex == 0 || VertexIndex == 1)
	{
		const uint BasePrimitiveIndex = PrimitiveIndex / HairStrandsVF.RaytracingProceduralSplits;

		Input.VertexId = BasePrimitiveIndex + VertexIndex;
		Input.PrimitiveId = PrimitiveIndex;
	}
#else
	{
		FTriangleBaseAttributes TriangleBase = LoadTriangleBaseAttributes(PrimitiveIndex);

		// The data in this buffer has 32 bits per float, while the data read from previous buffer only has 16 bits.
		// This mismatch will cause motion vector to be non-zero when there is no motion.
		// Input.Position = float4(TriangleBase.LocalPositions[VertexIndex], 1.0f);
		Input.VertexId = TriangleBase.Indices[VertexIndex];
		Input.PrimitiveId = PrimitiveIndex;
		Input.PrimitiveVertexId = VertexIndex;
	}
#endif // !ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR && !ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR
	
	const FVertexInfo VertexInfo = GetVertexInfo(Input);
	Input.Position = float4(GetVertexPosition(VertexInfo, true).Position, 1.f);

	// Note: GetInstanceUserData() stores the GPU-Scene instance ID
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	return Input;
}
#endif

/** 
 * Caches intermediates that would otherwise have to be computed multiple times.  Avoids relying on the compiler to optimize out redundant operations.
 */
struct FVertexFactoryIntermediates
{
	half3x3 TangentToLocal;
	half3x3 TangentToWorld;
	half TangentToWorldSign;
	uint HairControlPointId;
	float2 HairPrimitiveUV;
	float2 HairDimensions;  // This is kept on the intermediate as this is used in several vertex shader for the actual coverage.
	float  HairDensity;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

float3 GetPositionOffset()
{
	return ReadRenPositionOffset(HairStrandsVF.PositionOffsetBuffer, HairStrandsVF.RegisteredIndex);
}

float3 GetPrevPositionOffset()
{
	return ReadRenPrevPositionOffset(HairStrandsVF.PreviousPositionOffsetBuffer, HairStrandsVF.RegisteredIndex);
}

uint GetHairControlPointId(FVertexFactoryInput Input)
{
	return GetVertexInfo(Input).HairControlPointId;
}

FHairControlPoint GetVertexPosition(const FVertexInfo VertexInfo, bool bInvalidJointVertex=true)
{
	const float3 PositionOffset = GetPositionOffset();

	FHairControlPoint Out = ReadHairControlPoint(
		HairStrandsVF.PositionBuffer,
		VertexInfo.VertexIndex, 
		PositionOffset,
		HairStrandsVF.Radius,
		HairStrandsVF.RootScale,
		HairStrandsVF.TipScale);
	
	// Create a degenerated quad the end of each strand to cut between each strands
	// #hair_todo: This is not efficient for short strand like fur for instance. Need to revisit that at some point
#if RAYHITGROUPSHADER
	const bool bIsInvalidQuad = false; // can't hit an invalid quad in raytracing (filtered by the acceleration structure)
#elif USE_HAIR_TRIANGLE_STRIP
	const bool bIsInvalidQuad = (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_END) || VertexInfo.bForceInvalidQuad;
#else
	const bool bIsInvalidQuad = (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_START && VertexInfo.IsTip == 1) || (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_END && VertexInfo.IsTip == 0) || VertexInfo.bForceInvalidQuad;
#endif
	Out.Position = bIsInvalidQuad ? float3(INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT) : Out.Position;

	return Out;
}

FHairControlPoint GetVertexPosition(FVertexFactoryInput Input, bool bInvalidJointVertex=true)
{
	const FVertexInfo VertexInfo = GetVertexInfo(Input);
	return GetVertexPosition(VertexInfo, bInvalidJointVertex);
}

float GetWorldStrandRadius(FVertexFactoryInput Input)
{
	return GetVertexPosition(Input).WorldRadius;
}

float3 GetVertexPreviousPosition(FVertexFactoryInput Input, bool bInvalidJointVertex = true)
{
	const float3 PreviousPositionOffset = GetPrevPositionOffset();

	FVertexInfo VertexInfo = GetVertexInfo(Input);
	FHairControlPoint Out = ReadHairControlPoint(
		HairStrandsVF.PreviousPositionBuffer,
		VertexInfo.VertexIndex,
		PreviousPositionOffset,
		HairStrandsVF.Radius,
		HairStrandsVF.RootScale,
		HairStrandsVF.TipScale);

	// Create a degenerated quad the end of each strand to cut between each strands
	// #hair_todo: This is not efficient for short strand like fur for instance. Need to revisit that at some point
#if RAYHITGROUPSHADER
	const bool bIsInvalidQuad = false; // can't hit an invalid quad in raytracing (filtered by the acceleration structure)
#elif USE_HAIR_TRIANGLE_STRIP
	const bool bIsInvalidQuad = (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_END) || VertexInfo.bForceInvalidQuad;
#else
	const bool bIsInvalidQuad = (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_START && VertexInfo.IsTip == 1) || (bInvalidJointVertex && Out.Type == HAIR_CONTROLPOINT_END && VertexInfo.IsTip == 0) || VertexInfo.bForceInvalidQuad;
#endif
	Out.Position = bIsInvalidQuad ? float3(INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT) : Out.Position;

	return Out.Position;
}

// Segment UV coord of an hair segment. This is different from the UV coord of the hair strands
float2 GetSegmentUVCoord(FVertexFactoryInput Input)
{
	FVertexInfo VertexInfo = GetVertexInfo(Input);
#if RAYHITGROUPSHADER
	const float VCoord = 0.5; // TODO: find a way to define this?
#else
	const float VCoord = VertexInfo.IsLeft ? 0.0f : 1.0f;
#endif
	const float UCoord = VertexInfo.IsTip  ? 1.0f : 0.0f;
	return float2(UCoord, VCoord);
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

	half3 TangentToWorld0 = GetTangentToWorld0(Interpolants).xyz;
	half4 TangentToWorld2 = GetTangentToWorld2(Interpolants);
	Result.UnMirrored = TangentToWorld2.w;

	Result.TangentToWorld = AssembleTangentToWorldForHairStrand( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

	Result.TwoSidedSign = 1;
	Result.PrimitiveId = ToScalarMemory(GetPrimitiveId(Interpolants));
	Result.HairControlPointId = Interpolants.HairControlPointId;
	Result.HairPrimitiveUV = float2(Interpolants.HairPrimitiveUV);

	return Result;
}

half3x3 CalcTangentToWorldNoScale(FPrimitiveSceneData SceneData, half3x3 TangentToLocal)
{
	half3x3 LocalToWorld = DFToFloat3x3(SceneData.LocalToWorld);
	half3 InvScale = SceneData.InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;
	return mul(TangentToLocal, LocalToWorld); 
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, bool bInvalidJointVertex);
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates, true);
}
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FHairViewInfo HairInfo);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3 PositionInstanceSpace,
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	Result.PositionInstanceSpace = PositionInstanceSpace;
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	// does not handle instancing!
	Result.TangentToWorld = Intermediates.TangentToWorld;
	
	Result.PrevFrameLocalToWorld = Intermediates.SceneData.Primitive.PreviousLocalToWorld;
	Result.PreSkinnedPosition = GetVertexPreviousPosition(Input);
	Result.PreSkinnedNormal = TangentToLocal[2];

	Result.LWCData = MakeMaterialLWCData(Result);

	return Result;
}

FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	FHairViewInfo HairInfo,
	bool bIsPreviousFrame = false)
{
	float3 PositionInstanceSpace;
	if (bIsPreviousFrame)
	{
		PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates, HairInfo);
	}
	return GetMaterialVertexParameters(Input, Intermediates, WorldPosition, PositionInstanceSpace, TangentToLocal, bIsPreviousFrame);
}

FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	float3 PositionInstanceSpace;
	if (bIsPreviousFrame)
	{
		PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	return GetMaterialVertexParameters(Input, Intermediates, WorldPosition, PositionInstanceSpace, TangentToLocal, bIsPreviousFrame);
}

float4 CalcWorldPosition(float3 Position, FDFMatrix LocalToWorld)
{
	return DFTransformLocalToTranslatedWorld(Position, LocalToWorld, ResolvedView.PreViewTranslation);
}

half3x3 CalcTangentToLocal(uint VertexIndex, inout float TangentSign)
{
	const half3 TangentInputX = HairStrandsVF.TangentBuffer[VertexIndex * 2 + 0].xyz;
	const half4 TangentInputZ = HairStrandsVF.TangentBuffer[VertexIndex * 2 + 1];

	half3 TangentX = TangentBias(TangentInputX);
	half4 TangentZ = TangentBias(TangentInputZ);
	TangentSign = TangentZ.w;

	// derive the binormal by getting the cross product of the normal and tangent
	half3 TangentY = cross(TangentZ.xyz, TangentX) * TangentZ.w;

	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	half3x3 Result;
	Result[0] = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	Result[1] = TangentY;
	Result[2] = TangentZ.xyz;

	return Result;
}

half3x3 CalcTangentToLocal(FVertexFactoryInput Input, inout float TangentSign)
{
	FVertexInfo VertexInfo = GetVertexInfo(Input);
	return CalcTangentToLocal(VertexInfo.VertexIndex, TangentSign);
}

half3x3 CalcTangentToWorld(FPrimitiveSceneData SceneData, half3x3 TangentToLocal)
{
	return CalcTangentToWorldNoScale(SceneData, TangentToLocal);
}

float3 GetTangent(FVertexFactoryIntermediates Intermediates)
{
	// Hair shader expec the tangent to be stored in place of the normal. This is what the StrandHairComponent is filling in:
	// [0]: Real normal		| Should be Tangent
	// [1]: Bitangent		| Should be BiTangent
	// [2]: Real Tangent	| Should be Normal
	return Intermediates.TangentToWorld[2].xyz;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Raytracing custom intersection shader

#if RAYHITGROUPSHADER

#if USE_MATERIAL_INTERSECTION_SHADER


RAY_TRACING_ENTRY_INTERSECTION(MaterialIS)
{

	float3 RayOrg = ObjectRayOrigin();
	float3 RayDir = ObjectRayDirection();
	float TMin = RayTMin();
	float TMax = RayTCurrent();

	uint VertexIndex0 = PrimitiveIndex() / HairStrandsVF.RaytracingProceduralSplits;
	uint VertexIndex1 = VertexIndex0 + 1;
	if (HairStrandsVF.bCullingEnable)
	{
		VertexIndex0 = HairStrandsVF.CullingIndexBuffer[VertexIndex0];
		VertexIndex1 = HairStrandsVF.CullingIndexBuffer[VertexIndex1];
	}

	const float EffectiveRadius = HairStrandsVF.Radius * HairStrandsVF.RaytracingRadiusScale;
	const float3 PositionOffset = GetPositionOffset();
	FHairControlPoint P0 = ReadHairControlPoint(
		HairStrandsVF.PositionBuffer,
		VertexIndex0,
		PositionOffset,
		EffectiveRadius,
		HairStrandsVF.RootScale,
		HairStrandsVF.TipScale);
	FHairControlPoint P1 = ReadHairControlPoint(
		HairStrandsVF.PositionBuffer,
		VertexIndex1,
		PositionOffset,
		EffectiveRadius,
		HairStrandsVF.RootScale,
		HairStrandsVF.TipScale);

	// ray/ribbon intersection - this is very approximate compared to a real oriented cone, but sufficient for thin strands
	float3 pa = P0.Position;
	float3 pb = P1.Position;
	float ra = P0.WorldRadius;
	float rb = P1.WorldRadius;
	float3 ro = RayOrg;
	float3 rd = RayDir;
	float3 u = pb - pa;
	float3 w0 = pa - ro;
	float3 w1 = pb - ro;
	float3 n = cross(rd, u);
	float3 n1 = cross(rd, n);
	float sc = saturate(-dot(w0, n1) / dot(u, n1));
	float3 cp = lerp(w0, w1, sc);
	float rc = lerp(ra, rb, sc);
	float t = dot(cp, rd) / dot(rd, rd);
	float3 dp = t * rd - cp;
	float d2 = length2(dp);
	float r2 = rc * rc;
	if (d2 < r2 && t > 4. * rc)
	{
		// NOTE: the tangents are not exactly equal to 'u' at the extremeties due to quantization, could be avoided with better compression
		float3 ta = P0.Type == HAIR_CONTROLPOINT_START ? u : HairStrandsVF.TangentBuffer[VertexIndex0 * 2 + 1].xyz;
		float3 tb = P1.Type == HAIR_CONTROLPOINT_END   ? u : HairStrandsVF.TangentBuffer[VertexIndex1 * 2 + 1].xyz;
		// clip the rounded portion from the end each segment
		if ((dot(t * rd - w0, ta) * dot(ta, u) < 0)) return;
		if ((dot(t * rd - w1, tb) * dot(tb, u) > 0)) return;

		float h = sign(dot(dp, n)) * sqrt(saturate(d2 / r2));
		FRayTracingIntersectionAttributes Attributes = (FRayTracingIntersectionAttributes)0;
		Attributes.SetBarycentrics(float2(sc, 0.5 * h + 0.5));
		ReportHit(t, HIT_KIND_TRIANGLE_FRONT_FACE, Attributes);
	}
}

#endif // USE_MATERIAL_INTERSECTION_SHADER

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Material evaluation for raytracing

// define this to let shaders know this alternative method is available
#define VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS 1 

FMaterialPixelParameters GetMaterialPixelParameters(float3 RayOrigin, float3 RayDirection, float HitT, uint HitPrimitiveIndex, uint HitClusterID, FRayTracingIntersectionAttributes HitAttributes, uint HitKind, float4 SvPosition, out float3 WorldGeoNormal)
{
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if VF_USE_PRIMITIVE_SCENE_DATA
	FVertexFactoryInput Input;
	// Note: GetInstanceUserData() stores the GPU-Scene instance ID
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);
#endif

	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input); // NOTE: Input is not used when VF_USE_PRIMITIVE_SCENE_DATA == 0

	const float EffectiveRadius = HairStrandsVF.Radius * HairStrandsVF.RaytracingRadiusScale;

	const float2 Barycentrics = HitAttributes.GetBarycentrics();

	float UCoord = Barycentrics.x;
	float VCoord = Barycentrics.y;

#if ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR
	uint VertexIndex0 = HitPrimitiveIndex;
#elif ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR
	uint VertexIndex0 = HitPrimitiveIndex / HairStrandsVF.RaytracingProceduralSplits;
#else
	// fetch triangle
	const FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(HitPrimitiveIndex);

	// NOTE: This must match the tesselation pattern produced by SetBodySegment() in HairStrandsRaytracingGeometry.usf
	// TODO: is there a way we could handle end-caps elegantly?
	uint VertexIndex0 = Tri.Indices.z / 4; // last index always belongs to the "bottom" vertex of the hair segment

	const float3 Weights = float3(1 - Barycentrics.x - Barycentrics.y, Barycentrics.x, Barycentrics.y);
	UCoord = dot(float3(1, float((HitPrimitiveIndex % 2) == 0), 0), Weights);
#endif

	uint VertexIndex1 = VertexIndex0 + 1;
	if (HairStrandsVF.bCullingEnable)
	{
		VertexIndex0 = HairStrandsVF.CullingIndexBuffer[VertexIndex0];
		VertexIndex1 = HairStrandsVF.CullingIndexBuffer[VertexIndex1];
	}
	
#if !ENABLE_HAIRSTRANDS_PROCEDURAL_INTERSECTOR
	// Azimuth distance for procedural primitives are evaluated in intersection shader.
	// A different approach is used here for better precision
	{
		const float3 PositionOffset = GetPositionOffset();
		const FHairControlPoint P0 = ReadHairControlPoint(
			HairStrandsVF.PositionBuffer,
			VertexIndex0,
			PositionOffset,
			EffectiveRadius,
			HairStrandsVF.RootScale,
			HairStrandsVF.TipScale);
		const FHairControlPoint P1 = ReadHairControlPoint(
			HairStrandsVF.PositionBuffer,
			VertexIndex1,
			PositionOffset,
			EffectiveRadius,
			HairStrandsVF.RootScale,
			HairStrandsVF.TipScale);

		float3 A = DFDemote(CalcWorldPosition(P0.Position, SceneData.Primitive.LocalToWorld).xyz);
		float3 B = DFDemote(CalcWorldPosition(P1.Position, SceneData.Primitive.LocalToWorld).xyz);

		// approximate position along the width of the ribbon
		

		const float3 AxisIntersection = lerp(A, B, UCoord.x);
		const float3 IntersectedPoint = RayOrigin + RayDirection * HitT;
		const float3 ShadingNormal = normalize(IntersectedPoint - AxisIntersection);
		const float3 HairStrandDirection = normalize(B - A);

		const float3 Proj = normalize(RayDirection - dot(RayDirection, HairStrandDirection) * HairStrandDirection);
		const float3 ProjPerpendicular = cross(Proj, HairStrandDirection);
		const float h = dot(ShadingNormal, ProjPerpendicular);
		VCoord = 0.5f * h + 0.5f;
	}
#endif

	Result.HairControlPointId = VertexIndex0;
	Result.HairPrimitiveUV = float2(UCoord, VCoord);

	// Get tangent frame from each vertex
	float TangentSign_V0 = 0; half3x3 TangentToWorld_V0 = CalcTangentToWorld(SceneData.Primitive, CalcTangentToLocal(VertexIndex0, TangentSign_V0));
	float TangentSign_V1 = 0; half3x3 TangentToWorld_V1 = CalcTangentToWorld(SceneData.Primitive, CalcTangentToLocal(VertexIndex1, TangentSign_V1));
	// interpolate to current point within the segment
	float3 TangentToWorld0 = normalize(lerp(TangentToWorld_V0[0], TangentToWorld_V1[0], UCoord));
	float3 TangentToWorld2 = normalize(lerp(TangentToWorld_V0[2], TangentToWorld_V1[2], UCoord));
	float TangentSign = lerp(TangentSign_V0, TangentSign_V1, UCoord) * GetPrimitive_DeterminantSign_FromFlags(SceneData.Primitive.Flags);

	Result.TangentToWorld = AssembleTangentToWorldForHairStrand(TangentToWorld0, float4(TangentToWorld2, TangentSign));

	// compute a smooth ribbon normal based on the tangent vector
	float3 T = Result.TangentToWorld[2];
	float3 D = RayDirection;
	WorldGeoNormal = normalize(cross(T, normalize(cross(D, T))));

#if ENABLE_HAIRSTRANDS_LINEAR_SWEPT_SPHERE_INTERSECTOR
	Result.TwoSidedSign = 1.f;
#else
	Result.TwoSidedSign = sign(dot(RayDirection, WorldGeoNormal));
#endif

	Result.PrimitiveId = SceneData.PrimitiveId;

	return Result;
}

#endif // RAYHITGROUPSHADER

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Material evaluation for regular shader

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData			= VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.HairControlPointId= GetHairControlPointId(Input);
	Intermediates.HairPrimitiveUV	= GetSegmentUVCoord(Input);
	Intermediates.HairDensity		= HairStrandsVF.Density;
	Intermediates.HairDimensions	= float2(0.f, GetWorldStrandRadius(Input));

	float TangentSign;
	Intermediates.TangentToLocal = CalcTangentToLocal(Input, TangentSign);
	Intermediates.TangentToWorld = CalcTangentToWorld(Intermediates.SceneData.Primitive, Intermediates.TangentToLocal);
	Intermediates.TangentToWorldSign = TangentSign * GetPrimitive_DeterminantSign_FromFlags(Intermediates.SceneData.Primitive.Flags);
	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
half3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.TangentToLocal;
}

#if RAYHITGROUPSHADER

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcWorldPosition(Input.Position.xyz, Intermediates.SceneData.Primitive.LocalToWorld);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

#else

float4 ComputeViewAlignedWorldPosition(FVertexFactoryInput Input, float3 WorldTangent, float4 WorldPosition, float WorldStrandRadius, FHairViewInfo HairViewInfo)
{
	FVertexInfo VertexInfo = GetVertexInfo(Input);

	// Minimal radius to snap the strand to a sample/pixel center (to avoid aliasing)
	const float DistanceToCamera = length(HairViewInfo.TranslatedWorldCameraOrigin - WorldPosition.xyz);
	const float MinStrandHairRadius = ConvertGivenDepthRadiusForProjectionType(HairViewInfo.RadiusAtDepth1, DistanceToCamera, HairViewInfo.bIsOrthoView);

	const float3 ViewDir = -HairViewInfo.ViewForward;
	const float3 Right = normalize(cross(WorldTangent, ViewDir));
	const float3 OutWorldPosition = WorldPosition.xyz + (VertexInfo.IsLeft ? -Right : Right) * max(WorldStrandRadius, MinStrandHairRadius);
	
	return float4(OutWorldPosition, 1);
}

// @return translated world position 
// Specialized version of VertexFactoryGetWorldPosition & ComputeViewAlignedWorldPosition, for visibility VS shader.
float4 VertexFactoryGetWorldPosition_Visibility(FVertexFactoryInput Input, FHairViewInfo HairViewInfo, inout uint OutHairControlPointId)
{
	const FVertexInfo VertexInfo = GetVertexInfo(Input);
	OutHairControlPointId = VertexInfo.HairControlPointId;

	FHairControlPoint ControlPoint = GetVertexPosition(VertexInfo);

	FVertexFactoryIntermediates Intermediates = GetVertexFactoryIntermediates(Input);
	const float4 WorldPosition = CalcWorldPosition(ControlPoint.Position, Intermediates.SceneData.Primitive.LocalToWorld);
	const float3 WorldTangent = GetTangent(Intermediates);

	// Minimal radius to snap the strand to a sample/pixel center (to avoid aliasing)
	const float DistanceToCamera = length(HairViewInfo.TranslatedWorldCameraOrigin - WorldPosition.xyz);
	const float MinStrandHairRadius = ConvertGivenDepthRadiusForProjectionType(HairViewInfo.RadiusAtDepth1, DistanceToCamera);
	const float3 ViewDir = -HairViewInfo.ViewForward;
	const float3 Right = normalize(cross(WorldTangent, ViewDir));
	const float3 OutWorldPosition = WorldPosition.xyz + (VertexInfo.IsLeft ? -Right : Right) * max(ControlPoint.WorldRadius, MinStrandHairRadius);

	return float4(OutWorldPosition, 1);
}

// @return translated world position 
// Specialized version of VertexFactoryGetWorldPosition & ComputeViewAlignedWorldPosition, for voxelization VS shader.
float4 VertexFactoryGetWorldPosition_Voxelization(FVertexFactoryInput Input, FHairViewInfo HairViewInfo, inout float OutHairWorldRadius, inout float OutHairDensity)
{
	const FVertexInfo VertexInfo = GetVertexInfo(Input);

	FHairControlPoint ControlPoint = GetVertexPosition(VertexInfo);
	OutHairWorldRadius = ControlPoint.WorldRadius;
	OutHairDensity = HairStrandsVF.Density;

	FVertexFactoryIntermediates Intermediates = GetVertexFactoryIntermediates(Input);
	const float4 WorldPosition = CalcWorldPosition(ControlPoint.Position, Intermediates.SceneData.Primitive.LocalToWorld);
	const float3 WorldTangent = GetTangent(Intermediates);

	// Minimal radius to snap the strand to a sample/pixel center (to avoid aliasing)
	const float3 ViewDir = -HairViewInfo.ViewForward;
	const float3 Right = normalize(cross(WorldTangent, ViewDir));
	const float3 OutWorldPosition = WorldPosition.xyz + (VertexInfo.IsLeft ? -Right : Right) * max(ControlPoint.WorldRadius, HairViewInfo.RadiusAtDepth1);

	return float4(OutWorldPosition, 1);
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FHairViewInfo HairViewInfo)
{
	FHairControlPoint ControlPoint = GetVertexPosition(Input);
	const float3 VertexPosition = ControlPoint.Position;
	const float4 WorldPosition = CalcWorldPosition(VertexPosition, Intermediates.SceneData.Primitive.LocalToWorld);

	// Hair shader expect the WorldNormal to be the tangent vector
	const float3 WorldTangent = GetTangent(Intermediates);
	return ComputeViewAlignedWorldPosition(Input, WorldTangent, WorldPosition, ControlPoint.WorldRadius, HairViewInfo);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FHairViewInfo HairViewInfo)
{
	float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, Intermediates, HairViewInfo);
	float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(Intermediates.SceneData.Primitive.WorldToLocal, ResolvedView.PreViewTranslation);
	float4 LocalPosition = mul(TranslatedWorldPosition, TranslatedWorldToLocal);
	return LocalPosition.xyz;
}

// This function is referenced by several "system" shaders, but none of these shaders (apart HitProxy, are actually used)
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	const FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits, UseStableRasterization());

	FHairViewInfo HairViewInfo;
	HairViewInfo.TranslatedWorldCameraOrigin = ResolvedView.TranslatedWorldCameraOrigin;
	HairViewInfo.ViewForward = ResolvedView.ViewForward;
	HairViewInfo.RadiusAtDepth1 = HairRenderInfo.RadiusAtDepth1Primary;
	HairViewInfo.bIsOrthoView = HairRenderInfo.bIsOrthoView;

	return VertexFactoryGetWorldPosition(Input, Intermediates, HairViewInfo);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, Intermediates);
	float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(Intermediates.SceneData.Primitive.WorldToLocal, ResolvedView.PreViewTranslation);
	float4 LocalPosition = mul(TranslatedWorldPosition, TranslatedWorldToLocal);
	return LocalPosition.xyz;
}
#endif

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

	SetTangents(Interpolants, Intermediates.TangentToWorld[0], Intermediates.TangentToWorld[2], Intermediates.TangentToWorldSign);
	
	Interpolants.HairControlPointId = Intermediates.HairControlPointId;
	Interpolants.HairPrimitiveUV = Intermediates.HairPrimitiveUV;
	SetSceneData(Interpolants, Intermediates.SceneData);

	return Interpolants;
}

/** for depth-only pass (Not used by the actual hair shaders)*/
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input)
{
	return 0;
}

// @return translated world position (without quad extension/reorientation).This is used only for velocity computation
float4 VertexFactoryGetWorldPositionRaw(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, bool bInvalidJointVertex)
{
	return CalcWorldPosition(GetVertexPosition(Input, bInvalidJointVertex).Position, Intermediates.SceneData.Primitive.LocalToWorld);
}
float4 VertexFactoryGetWorldPositionRaw(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetWorldPositionRaw(Input, Intermediates, true);
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, bool bInvalidJointVertex)
{
	const float3 VertexPosition = GetVertexPreviousPosition(Input, bInvalidJointVertex);
	return DFTransformLocalToTranslatedWorld(VertexPosition, Intermediates.SceneData.Primitive.PreviousLocalToWorld, ResolvedView.PrevPreViewTranslation);
}
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetPreviousWorldPosition(Input, Intermediates, true);
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, bool bInvalidJointVertex)
{
	return GetVertexPreviousPosition(Input, bInvalidJointVertex);
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input)
{
	return float3(0, 0, 1); 
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float3(0,0,1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Regular material evaluation for raytracing

#if COMPUTESHADER
FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint PrimitiveIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input;

	Input.VertexId = PrimitiveIndex * 3 + VertexIndex;

	return Input;
}

uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif // COMPUTESHADER

#if NEEDS_VERTEX_FACTORY_INTERPOLATION

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
};

float2 VertexFactoryGetRayTracingTextureCoordinate(FVertexFactoryRayTracingInterpolants Interpolants)
{
#if NUM_MATERIAL_TEXCOORDS
	return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
#else // #if NUM_MATERIAL_TEXCOORDS
	return float2(0, 0);
#endif // #if NUM_MATERIAL_TEXCOORDS
}

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	// Default initialize. Otherwise, some graphics pipelines that
	// couple tessellation with geometry shaders won't write to all TEXCOORD semantics,
	// but read from them when <FVertexFactoryRayTracingInterpolants> is being copied as a whole.
	FVertexFactoryRayTracingInterpolants O = (FVertexFactoryRayTracingInterpolants)0;

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.InstanceId = a.InterpolantsVSToPS.InstanceId;
#if NEEDS_LIGHTMAP_COORDINATE
	O.InterpolantsVSToPS.LightmapDataIndex = a.InterpolantsVSToPS.LightmapDataIndex;
#endif
#endif	
	// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
	// interpolated normal from 'whatever' interpolation scheme we're using

	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
	INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
#endif
#if	NEEDS_PER_INSTANCE_PARAMS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.PerInstanceParams);
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapCoordinate);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
		for (int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++tc)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
		}
#elif USE_PARTICLE_SUBUVS
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[0]);
#endif

	O.InterpolantsVSToPS.HairControlPointId = a.InterpolantsVSToPS.HairControlPointId;
	INTERPOLATE_MEMBER(InterpolantsVSToPS.HairPrimitiveUV);
	return O;
}
#endif // NEEDS_VERTEX_FACTORY_INTERPOLATION

#if RAYHITGROUPSHADER && ALLOW_CUSTOM_RAYTRACING_HIT_TYPE
bool IsFrontFaceHit()
{
	// Hardware-accelerated LSS intersection tests or procedural tests won’t hit back faces.
	return true;
}
#endif

#include "/Engine/Private/VertexFactoryDefaultInterface.ush"
