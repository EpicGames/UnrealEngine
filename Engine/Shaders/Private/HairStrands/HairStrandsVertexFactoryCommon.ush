// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/HairStrandsDefinitions.h"
#include "/Engine/Private/HairStrands/HairStrandsPack.ush"

#ifndef HAIR_STRANDS_PARAMETERS
#define HAIR_STRANDS_PARAMETERS 0
#endif

#define HAIR_SIM  0
#define HAIR_REN  1
#define HAIR_CAR  2
#define HAIR_MAX_CARDS_LOD 8

#define HAIR_CURR 0
#define HAIR_PREV 1

float3 ReadPositionOffset(StructuredBuffer<float4> In, uint InIndex, uint InType, uint InTime, uint InLOD) 
{ 
	return In[0].xyz; 
}

void WritePositionOffset(RWStructuredBuffer<float4> Out, uint InIndex, uint InType, uint InTime, uint InLOD, float3 InPositionOffset) 
{ 
	Out[0] = float4(InPositionOffset, 1);
}

float3 ReadRenPositionOffset(StructuredBuffer<float4> In, uint InIndex) 
{ 
	return ReadPositionOffset(In, InIndex, HAIR_REN, HAIR_CURR, 0);
}

float3 ReadRenPrevPositionOffset(StructuredBuffer<float4> In, uint InIndex) 
{ 
	return ReadPositionOffset(In, InIndex, HAIR_REN, HAIR_PREV, 0);
}

float3 ReadSimPositionOffset(StructuredBuffer<float4> In, uint InIndex) 
{ 
	return ReadPositionOffset(In, InIndex, HAIR_SIM, HAIR_CURR, 0);
}

float3 ReadCardPositionOffset(StructuredBuffer<float4> In, uint InIndex, uint InLOD) 
{ 
	return ReadPositionOffset(In, InIndex, HAIR_CAR, HAIR_CURR, InLOD);
}


///////////////////////////////////////////////////////////////////////////////////////////////////
// Hair strands instance parameters

#define HAIR_STRANDS_INSTANCE_PARAMETERS(Prefix) \
	uint						Prefix##_RegisteredIndex; \
	uint						Prefix##_bCullingEnable; \
	uint						Prefix##_RaytracingProceduralSplits; \
	uint						Prefix##_GroupIndex; \
	uint						Prefix##_GroupCount; \
	float						Prefix##_Radius; \
	float						Prefix##_RootScale; \
	float						Prefix##_TipScale; \
	float						Prefix##_Length; \
	float						Prefix##_Density; \
	uint						Prefix##_Flags; \
	float3						Prefix##_PositionOffset; \
	uint						Prefix##_PointCount; \
	uint						Prefix##_CurveCount; \
	uint						Prefix##_bSimulation; \
	uint						Prefix##_bSingleGuide; \
	uint						Prefix##_bPointRemapping; \
	uint						Prefix##_bCurveRemapping; \
	uint						Prefix##_SourcePointCount; \
	uint						Prefix##_SourceCurveCount; \
	float4x4					Prefix##_LocalToWorldPrimitiveTransform; \
	float4x4					Prefix##_LocalToTranslatedWorldPrimitiveTransform; \
	uint4						Prefix##_CurveAttributeOffsets[HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT]; \
	uint4						Prefix##_PointAttributeOffsets[HAIR_POINT_ATTRIBUTE_OFFSET_COUNT]; \
	uint 						Prefix##_CurveAttributeIndexToChunkDivAsShift; \
	uint						Prefix##_CurveAttributeChunkElementCount; \
	uint						Prefix##_CurveAttributeChunkStrideInBytes; \
	uint						Prefix##_PointAttributeIndexToChunkDivAsShift; \
	uint						Prefix##_PointAttributeChunkElementCount; \
	uint						Prefix##_PointAttributeChunkStrideInBytes; \
	ByteAddressBuffer			Prefix##_PositionBuffer; \
	StructuredBuffer<float4>	Prefix##_PositionOffsetBuffer; \
	ByteAddressBuffer			Prefix##_CurveBuffer; \
	ByteAddressBuffer			Prefix##_PointToCurveBuffer; \
	ByteAddressBuffer 			Prefix##_CurveAttributeBuffer; \
	ByteAddressBuffer 			Prefix##_PointAttributeBuffer; \
	ByteAddressBuffer 			Prefix##_CurveMappingBuffer; \
	ByteAddressBuffer 			Prefix##_PointMappingBuffer; \
	Buffer<uint>				Prefix##_CullingIndirectBuffer; \
	Buffer<uint>				Prefix##_CullingIndexBuffer; \
	Buffer<uint> 				Prefix##_CullingIndirectBufferArgs; \
	float3 Prefix##_GetHairInstancePositionOffset() { return ReadRenPositionOffset(Prefix##_PositionOffsetBuffer, Prefix##_RegisteredIndex); }

#if HAIR_STRANDS_PARAMETERS == 1
	HAIR_STRANDS_INSTANCE_PARAMETERS(HairStrandsVF)
#endif