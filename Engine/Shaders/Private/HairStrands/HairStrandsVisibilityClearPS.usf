// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"


uint uClearValue; 
float fClearValue;

void ClearPS(
#if PERMUTATION_OUTPUT_FORMAT == 0
	out uint OutColor : SV_Target0
#elif PERMUTATION_OUTPUT_FORMAT == 1
	out uint2 OutColor : SV_Target0
#elif PERMUTATION_OUTPUT_FORMAT == 2
	out float OutColor : SV_Target0
#endif
)
{
#if PERMUTATION_OUTPUT_FORMAT == 0
	OutColor = uClearValue;
#elif PERMUTATION_OUTPUT_FORMAT == 1
	OutColor = uClearValue.xx;
#elif PERMUTATION_OUTPUT_FORMAT == 2
	OutColor = fClearValue;
#endif
}