// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Render modes
#define RENDER_MODE_TRANSMITTANCE 0
#define RENDER_MODE_PPLL 1
#define RENDER_MODE_MSAA_VISIBILITY 2
#define RENDER_MODE_TRANSMITTANCE_AND_HAIRCOUNT 3
#define RENDER_MODE_COMPUTE_RASTER 4

#define INVALID_TILE_OFFSET ~0

#define HAIR_VISIBILITY_GROUP_COUNT_WIDTH 64

#include "../PackUnpack.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"
#include "HairStrandsVisibilityCommonStruct.ush"

 ////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
 // General util functions

float3 EncodeTangent(float3 N)
{
	return N * 0.5 + 0.5;
}

float3 DecodeTangent(float3 N)
{
	return N * 2 - 1;
}

uint float4ToUint(float4 v)
{
	uint4 i = uint4(v.x * 255, v.y * 255, v.z * 255, v.w * 255);
	return (0xFF & i.w) << 24 | (0xFF & i.z) << 16 | (0xFF & i.y) << 8 | (0xFF & i.x);
}

float4 UintToFloat4(uint In)
{
	uint4 Out;
	Out.x = (0xFF &  In);
	Out.y = (0xFF & (In >> 8));
	Out.z = (0xFF & (In >> 16));
	Out.w = (0xFF & (In >> 24));
	return Out / 255.f;
}

uint Uint16ToUint32(uint2 In)
{
	return (In.x & 0xFFFF) | ((In.y & 0xFFFF) << 16);
}

uint2 Uint32ToUint16(uint In)
{
	uint2 A;
	A.x = In & 0xFFFF;
	A.y = (In >> 16) & 0xFFFF;
	return A;
}

uint Float16ToUint32(float2 In)
{
	return Uint16ToUint32(f32tof16(In));
}

float2 Uint32ToFloat16(uint In)
{
	return f16tof32(Uint32ToUint16(In));
}

uint To8bitCoverage(float Coverage)
{
	return min(uint(Coverage * 0x100), 0xFFu);
}

float From8bitCoverage(uint Coverage8bit)
{
	return float(Coverage8bit) / 255.f;
}

uint To16bitCoverage(float Coverage)
{
	return min(uint(Coverage * 0x10000u), 0xFFFFu);
}

float From16bitCoverage(uint Coverage16bit)
{
	return float(Coverage16bit) / float(0xFFFF);
}

uint3 QuantizeTo8Bits(float3 T)
{
	const float3 Quanta = saturate((T + float(1).xxx) * 0.5f) * 0xFF;
	return uint3(Quanta.x, Quanta.y, Quanta.z);
}

float3 From8bits(float3 In8bits)
{
	return (normalize(In8bits / 255.f) * 2) - float(1).xxx;
}

uint PackVelocity(float2 EncodedVelocity)
{
	return f32tof16(EncodedVelocity.x) << 16 | f32tof16(EncodedVelocity.y);
}

float2 UnpackVelocity(uint PackedVelocity)
{
	return float2(f16tof32(PackedVelocity >> 16), f16tof32(PackedVelocity));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Node Desc
// 
// * Max 67,108,864 total nodes (This is 32 nodes per pixel at 1080p if all pixel were covered)
// * Max 63 nodes per pixel
// 26bits for offset | 6 bits for count (max 63 nodes)
//TODO rename FHairSamplesDesc
struct FNodeDesc
{
	uint Offset;
	uint Count;
};

uint EncodeNodeDesc(const FNodeDesc Desc)
{
	return (Desc.Offset & 0x03FFFFFF) | ((Desc.Count & 0x3F) << 26);
}

FNodeDesc DecodeNodeDesc(uint In)
{
	FNodeDesc Out;
	Out.Offset = In & 0x03FFFFFF;
	Out.Count  = (In >> 26) & 0x3F;
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair Vis

uint PackHairVisDepthCoverage(float InDepth, float InCoverage)
{
	return (uint(InDepth * 0x00FFFFFF) << 8) | uint(0xFFu * saturate(InCoverage));
}

uint PackHairVisDepthCoverage(float InDepth, uint InCoverage8bits)
{
	return (uint(InDepth * 0x00FFFFFF) << 8) | min(0xFFu, InCoverage8bits);
}

uint PackHairVisControlPointMaterialId(uint ControlPointId, uint MaterialId)
{
	return (ControlPointId & 0x00FFFFFF) | ((MaterialId & 0xFF) << 24);
}

uint UnpackHairVisControlPointId(uint RawVis)
{
	return RawVis & 0x00FFFFFF;
}

uint UnpackHairVisMaterialId(uint RawVis)
{
	return (RawVis >> 24) & 0xFF;
}

float UnpackHairVisDepth(uint InPacked)
{
	return float(InPacked >> 8) / float(0x00FFFFFF);
}

float UnpackHairVisCoverage(uint InPacked)
{
	return float(InPacked & 0xFF) / float(0xFF);
}

uint UnpackHairVisCoverage8bits(uint InPacked)
{
	return InPacked & 0xFF;
}

uint GetInvalidHairControlPointId()
{
	return 0xFFFFFFFF;
}

void PatchPackedHairVisCoverage(inout FPackedHairVis Out, uint Coverage8bit)
{
	Out.Depth_Coverage8bit = (Out.Depth_Coverage8bit & (0x00FFFFFFu << 8)) | (Coverage8bit & 0xFFu);
}

struct FHairVis
{
	float	Depth;
	float   Coverage;	  // Coverage in float point, but with 8-bits precision. For convenience.
	uint	Coverage8bit;
	uint	ControlPointId;
	uint	MaterialId;
};

FPackedHairVis PackHairVis(const FHairVis In)
{
	FPackedHairVis Out;
	Out.Depth_Coverage8bit	   = PackHairVisDepthCoverage(In.Depth, In.Coverage8bit);
	Out.ControlPointID_MaterialID = PackHairVisControlPointMaterialId(In.ControlPointId, In.MaterialId);
	return Out;
}

FHairVis UnpackHairVis(const FPackedHairVis In)
{
	FHairVis Out;
	Out.Depth		 = UnpackHairVisDepth(In.Depth_Coverage8bit);
	Out.Coverage8bit = UnpackHairVisCoverage8bits(In.Depth_Coverage8bit);
	Out.Coverage	 = float(Out.Coverage8bit) / float(0xFF);
	Out.ControlPointId= UnpackHairVisControlPointId(In.ControlPointID_MaterialID);
	Out.MaterialId	 = UnpackHairVisMaterialId(In.ControlPointID_MaterialID);

	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair sample

// Footprint: 128bits | 16bytes
// Option to reduce the footprint: 
// * we don't use ControlPointId at the moment which is encoded on 3bytes
// * Deferred the evaluation of Coverage & material properties into some other pass?
struct FHairSample
{
	float	Depth;
	float3	Tangent;
	uint	Coverage8bit;
	uint	ControlPointId;
	uint	MacroGroupId;
	float3	BaseColor;
	float	Roughness;
	float	Specular;
	float	Backlit;
	uint	LightChannelMask;
	float3	Emissive;
	uint	Flags;
};

FPackedHairSample PackHairSample(const FHairSample In)
{
	FPackedHairSample Out;

	// Tangent and Coverage
	// Encode into 1 x 32bits uint
	// Coverage is clamped o 255 as we only have 8 bits to store its value
	const float3 T = saturate(EncodeTangent(In.Tangent));
	Out.Tangent_Coverage8bit =
		PackRGBA8(float4(T,0)) |
		(min(uint(0xFF), In.Coverage8bit) << 24);

	// ControlPointId and MacroGroupId
	// Encode into 1 x 32bits uint
	// ControlPointId is on 28bits | MacroGroupId is on 4bits
	Out.ControlPointID_MacroGroupID = 
		((In.ControlPointId  & 0x0FFFFFFF)) |
		((In.MacroGroupId & 0xF) << 28); // If this encoding change, ensure it conforms to MAX_HAIR_MACROGROUP_COUNT

	Out.Depth					= In.Depth;																			// 32bits float
	Out.BaseColor_Roughness		= float4ToUint(float4(sqrt(In.BaseColor), In.Roughness));							// 32bits uint
	Out.Data 					= 
		(PackR8(In.Specular)              )| 
		(PackR8(In.Backlit)          <<  8)| 
		((In.Flags & 0xFF)           << 16)|
		((In.LightChannelMask & 0x7) << 24);

	// Emissive is not packed/unpacked
	return Out;
}

FHairSample UnpackHairSample(const FPackedHairSample In)
{
	FHairSample Out;
	Out.Depth = In.Depth;

	Out.Tangent 	 = DecodeTangent(UnpackRGBA8(In.Tangent_Coverage8bit).xyz);
	Out.Coverage8bit = (In.Tangent_Coverage8bit >> 24) & 0xFF;

	Out.ControlPointId  = In.ControlPointID_MacroGroupID & 0x0FFFFFFF;
	Out.MacroGroupId = (In.ControlPointID_MacroGroupID>>28) & 0xF; // If this encoding change, ensure it conforms to MAX_HAIR_MACROGROUP_COUNT

	const float4 BaseColorAndRoughness = UintToFloat4(In.BaseColor_Roughness);
	Out.BaseColor			= BaseColorAndRoughness.xyz * BaseColorAndRoughness.xyz;
	Out.Roughness			= BaseColorAndRoughness.w;
	Out.Specular			= UnpackR8(In.Data & 0xFF);
	Out.Backlit				= UnpackR8((In.Data>>8) & 0xFF);
	Out.Flags 				= (In.Data>>16) & 0xFF;
	Out.LightChannelMask	= (In.Data>>24) & 0x7;

	// Emissive is not packed/unpacked
	return Out;
}

void PatchPackedHairSampleCoverage(inout FPackedHairSample Out, uint Coverage8bit)
{
	Out.Tangent_Coverage8bit = (Out.Tangent_Coverage8bit & 0x00FFFFFF) | (min(uint(0xFF), Coverage8bit) << 24);
}

uint GetPackedHairSampleCoverage8bit(in FPackedHairSample In)
{
	return 0xFF & (In.Tangent_Coverage8bit>>24);
}

uint2 GetHairSampleResolution(uint InNodeCount)
{
	uint Resolution = ceil(sqrt(InNodeCount));
	return uint2(Resolution, Resolution);
}

uint2 GetHairSampleCoord(uint InLinearCoord, uint2 InResolution)
{
	return uint2(InLinearCoord % InResolution.x, InLinearCoord / InResolution.x);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair velocity

// Return true if the underlying pixel need to run with TAA fast resolve. This is the case when 
// the velocity is larger than a certain threshold.
bool NeedFastResolve(float2 InEncodedVelocity, float2 InVelocity, float InVelocityThreshold)
{
	const float VelocityMagnitude = sqrt(dot(InVelocity, InVelocity));
	return InEncodedVelocity.x > 0 && VelocityMagnitude > InVelocityThreshold;
}

bool NeedFastResolve(float2 InEncodedVelocity, float InVelocityThreshold)
{
	const float2 Velocity = DecodeVelocityFromTexture(float4(InEncodedVelocity, 0.0, 0.0)).xy;
	const float VelocityMagnitude = sqrt(dot(Velocity, Velocity));
	return InEncodedVelocity.x > 0 && VelocityMagnitude > InVelocityThreshold;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair view info (used for overriding view constant/uniform buffer)

struct FHairRenderInfo
{
	float RadiusAtDepth1Primary;
	float RadiusAtDepth1Velocity;
	float VelocityMagnitudeScale;

	bool bIsGPUDriven;
	bool bIsOrthoView; // Only used for shadow view

	float3 ViewForward;
	float3 TranslatedWorldCameraOrigin;
};

FHairRenderInfo GetHairRenderInfo(float4 ViewHairRenderInfo, uint ViewHairRenderInfoBits, bool bUseScableRasterization=false)
{
	FHairRenderInfo Info;
	Info.RadiusAtDepth1Primary  = bUseScableRasterization ? ViewHairRenderInfo.y : ViewHairRenderInfo.x;
	Info.RadiusAtDepth1Velocity = ViewHairRenderInfo.z;
	Info.VelocityMagnitudeScale = ViewHairRenderInfo.w;

	const uint BitField = ViewHairRenderInfoBits;
	Info.bIsOrthoView = (BitField & 0x1) != 0;
	Info.bIsGPUDriven = (BitField & 0x2) != 0;

	return Info;
}
 
struct FHairViewInfo
{
	float3 TranslatedWorldCameraOrigin;
	float3 ViewForward;
	float  RadiusAtDepth1;
	bool   bIsOrthoView;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair transmittance/coverage

float TransmittanceToCoverage(float InTransmittance, float InCoverageThreashold)
{
	return saturate(min(1 - InTransmittance, 1) / InCoverageThreashold);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Hair PPLL

FPackedHairVisPPLL PackHairVisPPLL(
	float	Depth, 
	float	Coverage,
	uint	ControlPointId,
	uint	MaterialId,
	uint	NextNodeIndex)
{
	FHairVis S;
	S.Depth = Depth;
	S.Coverage8bit = To8bitCoverage(Coverage); // To16bitCoverage(CoverageFullPrecision)
	S.ControlPointId = ControlPointId;
	S.MaterialId = MaterialId;
	const FPackedHairVis PackedS = PackHairVis(S);

	FPackedHairVisPPLL Out;
	Out.Depth_Coverage8bit = PackedS.Depth_Coverage8bit;
	Out.ControlPointID_MaterialID = PackedS.ControlPointID_MaterialID;
	Out.NextNodeIndex = NextNodeIndex;
	return Out;
}

FHairVis UnpackHairVisPPLL(FPackedHairVisPPLL In)
{
	FPackedHairVis Temp;
	Temp.Depth_Coverage8bit = In.Depth_Coverage8bit;
	Temp.ControlPointID_MaterialID = In.ControlPointID_MaterialID;
	return UnpackHairVis(Temp);
}

FPackedHairVis ConvertToPackedHairVis(FPackedHairVisPPLL In)
{
	FPackedHairVis Out;
	Out.Depth_Coverage8bit = In.Depth_Coverage8bit;
	Out.ControlPointID_MaterialID = In.ControlPointID_MaterialID;
	return Out;
}

 ////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
 // Coverage and hair count

uint PackCoverageAndHairCount(float InCoverage)
{
	// Assume the storage is enough for not overflowing
	// Encode the strands_radius/pixel_radius ratio into 8bits, and this can accumulate up to 256 element with coverage of 1
	//return (InHairCount<<16) | uint(saturate(InCoverage) * 0xFF);
	return uint(saturate(InCoverage) * 1000.f);
}

float UnpackCoverageAndHairCount(uint InPacked)
{
	return float(InPacked) / 1000.f;
}

