// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FPackedHairSample
{
	float Depth;
	uint  ControlPointID_MacroGroupID;
	uint  Tangent_Coverage8bit;
	uint  BaseColor_Roughness;
	uint  Data;
};

struct FPackedHairVis
{
	uint Depth_Coverage8bit;
	uint ControlPointID_MaterialID;
};

struct FPackedHairVisPPLL
{
	uint Depth_Coverage8bit;
	uint ControlPointID_MaterialID;
	uint NextNodeIndex;
};