// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush" 
#include "HairStrandsTileCommon.ush"

#if PERMUTATION_GROUPSIZE ==32
	#define TILE_PIXEL_SIZE_X 8
	#define TILE_PIXEL_SIZE_Y 4
#elif PERMUTATION_GROUPSIZE == 64
	#define TILE_PIXEL_SIZE_X 8
	#define TILE_PIXEL_SIZE_Y 8
#else
	#error Unknown group size	
#endif

int2 OutputResolution;
int2 ResolutionOffset;
uint MaxNodeCount;
uint bSortSampleByDepth;
float DepthTheshold; 
float CosTangentThreshold;
float CoverageThreshold; // Allow to scale the transmittance to compensate its asymptotic behavior 
uint VelocityType;

RWStructuredBuffer<uint> OutCompactNodeCounter;
RWTexture2D<uint> OutCompactNodeIndex;
RWStructuredBuffer<FPackedHairVis> OutCompactNodeVis;
RWBuffer<uint2> OutCompactNodeCoord;
RWTexture2D<float> OutCoverageTexture;

groupshared uint AllocationNodeCount;
groupshared uint AllocationNodeOffset;


struct FSampleSetDesc
{
	uint   UniqueSampleCount;
	uint   ValidSampleCount;
	uint   HairSampleCount;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////
// PPLL
///////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if PERMUTATION_PPLL
#define PPLL_MAX_RENDER_NODE_COUNT PERMUTATION_PPLL

Buffer<uint> TileCountBuffer;
Buffer<uint2> TileDataBuffer;

int2 TileCountXY;
Texture2D<uint> PPLLCounter;
Texture2D<uint> PPLLNodeIndex;
StructuredBuffer<FPackedHairVisPPLL> PPLLNodeData;


[numthreads(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y, 1)]
void MainCS(
	uint GroupIndex : SV_GroupIndex, 
	uint3 GroupId : SV_GroupID, 
	uint3 GroupThreadId : SV_GroupThreadID, 
	uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TileSize = HAIR_TILE_SIZE;
	const uint TileCount = TileCountBuffer[HAIRTILE_HAIR_ALL];
	const uint LinearIndex = GroupId.x + GroupId.y * TileCountXY.x;
	if (LinearIndex >= TileCount)
	{
		return;
	}
	const uint2 TileCoord = TileDataBuffer[LinearIndex];
	const uint2 PixelCoord = TileCoord * TileSize + GroupThreadId.xy;

	const uint FirstNodeIndex = PPLLNodeIndex[PixelCoord];
	float PixelCoverage = 0;
	
	const bool bExecute = FirstNodeIndex != 0xFFFFFFFF && all(PixelCoord < uint2(OutputResolution));

	uint SortedIndex[PPLL_MAX_RENDER_NODE_COUNT];
	float SortedDepth[PPLL_MAX_RENDER_NODE_COUNT];
	for (int NodeIndex = 0; NodeIndex < PPLL_MAX_RENDER_NODE_COUNT; ++NodeIndex)
	{
		SortedDepth[NodeIndex] = 0.0f;
	}

	float TotalTransmittance = 1.0f;
	uint PixelNodeCount = 0;
	if (bExecute)
	{
		//////////
		// First pass: sort PPLL element into nodes we are going to render
		uint NodeIndex = FirstNodeIndex;
		while (NodeIndex != 0xFFFFFFFF)
		{
			const FPackedHairVisPPLL PackedHairVis = PPLLNodeData[NodeIndex];
			const FHairVis HairVis = UnpackHairVisPPLL(PackedHairVis);

			TotalTransmittance *= 1.0f - HairVis.Coverage;

			int InsertPos;
			for (InsertPos = 0; InsertPos < PPLL_MAX_RENDER_NODE_COUNT; ++InsertPos)
			{
				if (HairVis.Depth > SortedDepth[InsertPos])
				{
					// The new node is closer
					break;
				}
			}

			if (InsertPos == PPLL_MAX_RENDER_NODE_COUNT)
			{
				// TODOAGREGATE Need to merge the node into last node
				// ==> cull it out for now
			}
			else
			{
				// Shift existing nodes from the end
				for (int Shift = PPLL_MAX_RENDER_NODE_COUNT - 1; Shift > InsertPos; --Shift) // TODO use PixelNodeCount in place of PPLL_MAX_RENDER_NODE_COUNT
				{
					SortedIndex[Shift] = SortedIndex[Shift-1];
					SortedDepth[Shift] = SortedDepth[Shift-1];
				}

				// TODOAGREGATE merge last node into agregated material

				// Insert new node
				SortedIndex[InsertPos] = NodeIndex;
				SortedDepth[InsertPos] = HairVis.Depth;

				PixelNodeCount++;
			}

			NodeIndex = PackedHairVis.NextNodeIndex;
		}
	}

	// Initialise group allocation node count
	if (GroupIndex == 0)
	{
		AllocationNodeCount = 0;
		AllocationNodeOffset= 0;
	}
	GroupMemoryBarrierWithGroupSync();
	// Now notify how many nodes this thread wants to allocate
	uint PixelDataSubOffsetInStorage = 0; 
	uint PixelRenderNodecount = min(PixelNodeCount, uint(PPLL_MAX_RENDER_NODE_COUNT));
	InterlockedAdd(AllocationNodeCount, PixelRenderNodecount, PixelDataSubOffsetInStorage);
	GroupMemoryBarrierWithGroupSync();
	// Now allocate all the nodes for this group contiguously in memory
	if (GroupIndex == 0 && AllocationNodeCount > 0)
	{
		InterlockedAdd(OutCompactNodeCounter[0], AllocationNodeCount, AllocationNodeOffset);
	}
	GroupMemoryBarrierWithGroupSync();
	uint OffsetInStorage = AllocationNodeOffset + PixelDataSubOffsetInStorage;

	if (bExecute)
	{
		//////////
		// Second pass: compute total coverage for validated nodes we are going to render
		float ValidPixelSampleTotalCoverage = 0.0f;
		float SortedCoverage[PPLL_MAX_RENDER_NODE_COUNT];
		float TotalSortedTransmittance = 1.0f;
		for (uint i = 0; i < PixelRenderNodecount; ++i)
		{
			const uint NodePPLLIndex	 = SortedIndex[i];

			const FPackedHairVisPPLL PackedHairVis = PPLLNodeData[NodePPLLIndex];
			const FHairVis HairVis = UnpackHairVisPPLL(PackedHairVis);
			const float Coverage = HairVis.Coverage; // This should be 16bits, instead of 8bits for better precision

			// Update current node coverage as a function of previous nodes coverage
			SortedCoverage[i] = TotalSortedTransmittance * Coverage;
			
			// Update transmittance for the next strands
			TotalSortedTransmittance *= 1.0f - Coverage;

			// Accumulate total coverage.
			ValidPixelSampleTotalCoverage += SortedCoverage[i];
		}

		//////////
		// Third pass: write out compact nodes for rendering
		for (uint j = 0; j < PixelRenderNodecount; ++j)
		{
			const uint NodePPLLIndex = SortedIndex[j];
			FPackedHairVis PackedHairVis = ConvertToPackedHairVis(PPLLNodeData[NodePPLLIndex]);

			// Coverage8bit is a weight normalising to 1 the contribution of all the compacted samples. Because later it is weighted by PixelCoverage.
			// Patch the coverage on the out node
			const float PatchedCoverage8bit = To8bitCoverage(SortedCoverage[j] / float(ValidPixelSampleTotalCoverage));
			PatchPackedHairVisCoverage(PackedHairVis, PatchedCoverage8bit);
			
			// TODOAGREGATE if last, Create FPackedHairSample from aggregated  data
			OutCompactNodeVis[OffsetInStorage + j] = PackedHairVis;
			OutCompactNodeCoord[OffsetInStorage + j] = PixelCoord;

		}

		PixelCoverage = TransmittanceToCoverage(TotalTransmittance, CoverageThreshold);	
	}

	FNodeDesc NodeDesc;
	NodeDesc.Count = PixelRenderNodecount;
	NodeDesc.Offset = OffsetInStorage;
	OutCompactNodeIndex[PixelCoord] = EncodeNodeDesc(NodeDesc);

	OutCoverageTexture[PixelCoord] = PixelCoverage;
}

#else // PERMUTATION_PPLL or PERMUTATION_VISIBILITY

///////////////////////////////////////////////////////////////////////////////////////////////////////////////
// MSAA visiblity buffer
///////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifndef PERMUTATION_MSAACOUNT
	#error PERMUTATION_MSAACOUNT is not defined
#endif

#define MERGE_SAMPLE 0

Texture2D<float> SceneDepthTexture;

#if PERMUTATION_MSAACOUNT > 1
void ComputeUniqueSamplesWithoutDepthTest(
	inout uint4 OutSamples[PERMUTATION_MSAACOUNT],
	inout FSampleSetDesc OutSet,
	const uint2 PixelCoord,
	const float OpaqueDepth,
	Texture2DMS<uint, PERMUTATION_MSAACOUNT>  InMSAA_IDTexture,
	Texture2DMS<float, PERMUTATION_MSAACOUNT>  InMSAA_DepthTexture)
{
	OutSet.UniqueSampleCount = 0;
	OutSet.ValidSampleCount = 0;
	OutSet.HairSampleCount = PERMUTATION_MSAACOUNT;

	for (uint SampleIt = 0; SampleIt < OutSet.HairSampleCount; ++SampleIt)
	{
		// Note: InMSAA_IDTexture contains both the primitive ID and tha material ID. However 
		// the material ID is constant along the primitive, so it is correct to use this as a 
		// sorting/deduplication key
		const uint HairControlPointId = InMSAA_IDTexture.Load(PixelCoord, SampleIt);
		const bool bIsValid = HairControlPointId != GetInvalidHairControlPointId();
		if (!bIsValid)
			continue;

		const float SampleDepth = InMSAA_DepthTexture.Load(PixelCoord, SampleIt);
		if (OpaqueDepth > SampleDepth) // Reverse-Z
			continue;

		++OutSet.ValidSampleCount;
	#if MERGE_SAMPLE 
		const float SceneDepth  = ConvertFromDeviceZ(SampleDepth);
	#endif
		bool bAlreadyExist = false;
		for (uint UniqueIt = 0; UniqueIt < OutSet.UniqueSampleCount; ++UniqueIt)
		{

		#if MERGE_SAMPLE
			const float UniqueDepth = asfloat(OutSamples[UniqueIt].w);
			const float UniqueSceneDepth = ConvertFromDeviceZ(UniqueDepth);
			const bool bIsSimilar = 
				HairControlPointId == OutSamples[UniqueIt].x ||
				abs(UniqueSceneDepth - SceneDepth) < DepthTheshold;
		#else
			const bool bIsSimilar = HairControlPointId == OutSamples[UniqueIt].x;
		#endif
			if (bIsSimilar)
			{
				OutSamples[UniqueIt].y += 1;

				// Update the unique sample with the closest depth
				const uint IntDepth = asuint(SampleDepth);
				if (IntDepth > OutSamples[UniqueIt].w)
				{
				#if MERGE_SAMPLE
					OutSamples[UniqueIt].x = HairControlPointId;
				#endif
					OutSamples[UniqueIt].z = SampleIt;
					OutSamples[UniqueIt].w = asuint(SampleDepth);
				}

				bAlreadyExist = true;
				break;
			}
		}
		
		if (!bAlreadyExist)
		{
			OutSamples[OutSet.UniqueSampleCount].x = HairControlPointId;
			OutSamples[OutSet.UniqueSampleCount].y = 1;
			OutSamples[OutSet.UniqueSampleCount].z = SampleIt;
			OutSamples[OutSet.UniqueSampleCount].w = asuint(SampleDepth);
			++OutSet.UniqueSampleCount;
		}
	}

	// Sort sample from closer to further. This is used later for updating sample coverage 
	// based on ordered transmittance. See HairStrandsVisibilityComputeSampleCoverage.usf for more details.
	if (bSortSampleByDepth > 0)
	{
		for (uint i = 0; i < OutSet.UniqueSampleCount; ++i)
		{
			const uint DepthI = OutSamples[i].w;
			for (uint j = i+1; j < OutSet.UniqueSampleCount; ++j)
			{
				const uint DepthJ = OutSamples[j].w;
				if (DepthJ > DepthI)
				{
					uint4 Temp = OutSamples[i];
					OutSamples[i] = OutSamples[j];
					OutSamples[j] = Temp;
				}
			}
		}
	}
}

Texture2DMS<uint, PERMUTATION_MSAACOUNT> MSAA_IDTexture;
Texture2DMS<float, PERMUTATION_MSAACOUNT> MSAA_DepthTexture;
#else

void ComputeUniqueSamplesWithoutDepthTest(
	inout uint4 OutSamples[PERMUTATION_MSAACOUNT],
	inout FSampleSetDesc OutSet,
	const uint2 PixelCoord,
	const float OpaqueDepth,
	Texture2D<uint>  InMSAA_IDTexture,
	Texture2D<float>  InMSAA_DepthTexture)
{
	OutSet.UniqueSampleCount = 0;
	OutSet.ValidSampleCount = 0;
	OutSet.HairSampleCount = PERMUTATION_MSAACOUNT;

	const uint HairControlPointId = InMSAA_IDTexture.Load(uint3(PixelCoord, 0));
	const bool bIsValid = HairControlPointId != GetInvalidHairControlPointId();
	if (!bIsValid)
		return;

	const float SampleDepth = InMSAA_DepthTexture.Load(uint3(PixelCoord, 0));
	if (OpaqueDepth > SampleDepth) // Reverse-Z
		return;

	++OutSet.ValidSampleCount;
	OutSamples[0].x = HairControlPointId;
	OutSamples[0].y = 1;
	OutSamples[0].z = 0;
	OutSamples[0].w = asuint(SampleDepth);
	++OutSet.UniqueSampleCount;
}

Texture2D<uint> MSAA_IDTexture;
Texture2D<float> MSAA_DepthTexture;

uint2  GetIDTexture(uint2 PixelCoord, uint SampleIt) { return MSAA_IDTexture.Load(uint3(PixelCoord, 0)); }
float  GetDepthTexture(uint2 PixelCoord, uint SampleIt) { return MSAA_DepthTexture.Load(uint3(PixelCoord, 0)); }

#endif // PERMUTATION_MSAACOUNT>1

int2 TileCountXY;
uint TileSize;
Buffer<uint> TileCountBuffer;
Buffer<uint2> TileDataBuffer;

Texture2D<float> ViewTransmittanceTexture;

[numthreads(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y, 1)]
void MainCS(
	uint  GroupIndex		: SV_GroupIndex, 
	uint3 GroupId			: SV_GroupID, 
	uint3 GroupThreadId		: SV_GroupThreadID, 
	uint3 DispatchThreadId	: SV_DispatchThreadID)
{
	if (GroupIndex == 0)
	{
		AllocationNodeCount = 0;
		AllocationNodeOffset = 0;
	}
	const uint TileCount = TileCountBuffer[HAIRTILE_HAIR_ALL];
	const uint LinearIndex = GroupId.x + GroupId.y * TileCountXY.x;
	if (LinearIndex >= TileCount)
	{
		return;
	}
	const uint2 TileCoord = TileDataBuffer[LinearIndex];
	uint2 PixelCoord = TileCoord * TileSize + GroupThreadId.xy;

	const bool bIsValid = all(PixelCoord < View.ViewRectMinAndSize.zw);
	if (!bIsValid)
	{
		PixelCoord = uint2(0, 0);
	}
	PixelCoord += View.ViewRectMinAndSize.xy;

	FSampleSetDesc SampleDesc;

	const float OpaqueDepth = SceneDepthTexture.Load(uint3(PixelCoord, 0)).r;
	uint4 Samples[PERMUTATION_MSAACOUNT];		// x:ControlPointIdId|MaterialId, y:Weight, z:SampleIt, w:Depth (as uint)
	ComputeUniqueSamplesWithoutDepthTest(
		Samples,
		SampleDesc,
		PixelCoord,
		OpaqueDepth,
		MSAA_IDTexture, 
		MSAA_DepthTexture);

	FNodeDesc NodeDesc;
	NodeDesc.Count = SampleDesc.UniqueSampleCount;
	NodeDesc.Offset = 0;

	if (NodeDesc.Count > 0)
	{
		InterlockedAdd(AllocationNodeCount, NodeDesc.Count, NodeDesc.Offset);
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupIndex == 0 && AllocationNodeCount > 0)
	{
		InterlockedAdd(OutCompactNodeCounter[0], AllocationNodeCount, AllocationNodeOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// Allocate node space
	float PixelCoverage = 0;
	if (NodeDesc.Count > 0)
	{
		NodeDesc.Offset += AllocationNodeOffset;

		// Store final sort node data 
		if (NodeDesc.Offset + NodeDesc.Count < MaxNodeCount)
		{
			float ClosestDepth = 0; // Inverse-Z
			for (uint OutIndex = 0; OutIndex < NodeDesc.Count; ++OutIndex)
			{
				// VisibilityData.Coverage8bit is a weight normalising to 1 the contribution of all the compacted samples. Because later it is weighted by PixelCoverage.
				FHairVis OutNodeVis;
				OutNodeVis.ControlPointId= UnpackHairVisControlPointId(Samples[OutIndex].x);
				OutNodeVis.Depth		= asfloat(Samples[OutIndex].w); // MSAA_DepthTexture.Load(PixelCoord, Samples[OutIndex].z);
				OutNodeVis.Coverage8bit = To8bitCoverage(Samples[OutIndex].y / float(SampleDesc.ValidSampleCount));
				OutNodeVis.MaterialId	= UnpackHairVisMaterialId(Samples[OutIndex].x);

				ClosestDepth = max(ClosestDepth, OutNodeVis.Depth); // Inverse-Z

				const uint StoreIndex = NodeDesc.Offset + OutIndex;
				OutCompactNodeVis[StoreIndex] = PackHairVis(OutNodeVis);
				OutCompactNodeCoord[StoreIndex] = PixelCoord;
			}

			NodeDesc.Count = NodeDesc.Count;
			PixelCoverage = TransmittanceToCoverage(ViewTransmittanceTexture.Load(uint3(PixelCoord, 0)), CoverageThreshold);
		}
	}

	OutCompactNodeIndex[PixelCoord] = EncodeNodeDesc(NodeDesc);
	OutCoverageTexture[PixelCoord] = PixelCoverage;
}

#endif // PERMUTATION_PPLL or PERMUTATION_VISIBILITY
