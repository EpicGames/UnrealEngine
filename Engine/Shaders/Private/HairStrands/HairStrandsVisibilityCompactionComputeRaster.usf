// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush" 
#include "HairStrandsTileCommon.ush" 

#if PERMUTATION_GROUPSIZE == 0
#define TILE_PIXEL_SIZE_X 8
#define TILE_PIXEL_SIZE_Y 4
#else
#define TILE_PIXEL_SIZE_X 8
#define TILE_PIXEL_SIZE_Y 8
#endif

#ifndef PERMUTATION_MULTI_SAMPLE_COUNT
#define PERMUTATION_MULTI_SAMPLE_COUNT 1
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

//TODO: re-enable support for 64 bit visibilty buffer format?
/*
#if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS && COMPILER_SUPPORTS_ULONG_TYPES
#define PackedType UlongType
uint2 UnpackData(PackedType In)
{
	return UnpackUlongType(In);
}
#else
#define PackedType uint2
uint2 UnpackData(PackedType In)
{
	return In;
}
#endif
*/

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

int2 OutputResolution;
uint MaxNodeCount;
uint SamplerPerPixel;
float CoverageThreshold;
uint bSortSampleByDepth;

RWStructuredBuffer<uint> 			OutCompactNodeCounter;
RWTexture2D<uint> 					OutCompactNodeIndex;
RWStructuredBuffer<FPackedHairVis> 	OutCompactNodeVis;
RWBuffer<uint2>						OutCompactNodeCoord;
RWTexture2D<float> 					OutCoverageTexture;

groupshared uint AllocationNodeCount;
groupshared uint AllocationNodeOffset;


struct FSampleSetDesc
{
	uint   UniqueSampleCount;
	uint   ValidSampleCount;
	uint   HairSampleCount;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Compute raster visibility buffer compaction
///////////////////////////////////////////////////////////////////////////////////////////////////////////////

Texture2DArray<uint>  DepthCovTexture;
Texture2DArray<uint>  PrimMatTexture;
Texture2D<uint>  HairCountTexture;

int2 TileCountXY;
uint TileSize;
Buffer<uint> TileCountBuffer;
Buffer<uint2> TileDataBuffer;

#define MERGE_SAMPLE 0

void ComputeUniqueSamples(const uint2 PixelCoord, out uint4 OutSamples[PERMUTATION_MULTI_SAMPLE_COUNT], out FSampleSetDesc OutSet)
{
	OutSet.UniqueSampleCount = 0;
	OutSet.ValidSampleCount = 0;
	OutSet.HairSampleCount = PERMUTATION_MULTI_SAMPLE_COUNT;

	for (uint SampleIt = 0; SampleIt < OutSet.HairSampleCount; ++SampleIt)
	{
		// Note: PrimMatTexture contains both the primitive ID and the material ID. However 
		// the material ID is constant along the primitive, so it is correct to use this as a 
		// sorting/deduplication key
		const uint HairControlPointId = PrimMatTexture.Load(uint4(PixelCoord, SampleIt, 0));
		const bool bIsValid = HairControlPointId != GetInvalidHairControlPointId();
		if (!bIsValid)
		{
			continue;
		}

		const float SampleDepth = UnpackHairVisDepth(DepthCovTexture.Load(uint4(PixelCoord, SampleIt, 0)));

		++OutSet.ValidSampleCount;
#if MERGE_SAMPLE 
		const float SceneDepth = ConvertFromDeviceZ(SampleDepth);
#endif
		bool bAlreadyExist = false;
		for (uint UniqueIt = 0; UniqueIt < OutSet.UniqueSampleCount; ++UniqueIt)
		{

#if MERGE_SAMPLE
			const float UniqueDepth = asfloat(OutSamples[UniqueIt].w);
			const float UniqueSceneDepth = ConvertFromDeviceZ(UniqueDepth);
			const bool bIsSimilar =
				HairControlPointId == OutSamples[UniqueIt].x ||
				abs(UniqueSceneDepth - SceneDepth) < DepthTheshold;
#else
			const bool bIsSimilar = HairControlPointId == OutSamples[UniqueIt].x;
#endif
			if (bIsSimilar)
			{
				OutSamples[UniqueIt].y += 1;

				// Update the unique sample with the closest depth
				const uint IntDepth = asuint(SampleDepth);
				if (IntDepth > OutSamples[UniqueIt].w)
				{
#if MERGE_SAMPLE
					OutSamples[UniqueIt].x = HairControlPointId;
#endif
					OutSamples[UniqueIt].z = SampleIt;
					OutSamples[UniqueIt].w = asuint(SampleDepth);
				}

				bAlreadyExist = true;
				break;
			}
		}

		if (!bAlreadyExist)
		{
			OutSamples[OutSet.UniqueSampleCount].x = HairControlPointId;
			OutSamples[OutSet.UniqueSampleCount].y = 1;
			OutSamples[OutSet.UniqueSampleCount].z = SampleIt;
			OutSamples[OutSet.UniqueSampleCount].w = asuint(SampleDepth);
			++OutSet.UniqueSampleCount;
		}
	}

	// Sort sample from closer to further. This is used later for updating sample coverage 
	// based on ordered transmittance. See HairStrandsVisibilityComputeSampleCoverage.usf for more details.
	if (bSortSampleByDepth > 0)
	{
		for (uint i = 0; i < OutSet.UniqueSampleCount; ++i)
		{
			const uint DepthI = OutSamples[i].w;
			for (uint j = i + 1; j < OutSet.UniqueSampleCount; ++j)
			{
				const uint DepthJ = OutSamples[j].w;
				if (DepthJ > DepthI)
				{
					uint4 Temp = OutSamples[i];
					OutSamples[i] = OutSamples[j];
					OutSamples[j] = Temp;
				}
			}
		}
	}
}

[numthreads(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (GroupIndex == 0)
	{
		AllocationNodeCount = 0;
		AllocationNodeOffset = 0;
	}

	const uint TileCount = TileCountBuffer[HAIRTILE_HAIR_ALL];
	const uint LinearIndex = GroupId.x + GroupId.y * TileCountXY.x;
	if (LinearIndex >= TileCount)
	{
		return;
	}
	const uint2 TileCoord = TileDataBuffer[LinearIndex];
	uint2 PixelCoord = TileCoord * TileSize + GroupThreadId.xy;

	if (PixelCoord.x >= uint(OutputResolution.x) || PixelCoord.y >= uint(OutputResolution.y))
	{
		PixelCoord = uint2(0, 0);
	}

	FSampleSetDesc SampleDesc;
	uint4 Samples[PERMUTATION_MULTI_SAMPLE_COUNT]; // x:ControlPointId|MaterialId, y:Weight, z:SampleIt, w:Depth (as uint)

	ComputeUniqueSamples(PixelCoord, Samples, SampleDesc);

	FNodeDesc NodeDesc;
	NodeDesc.Count = SampleDesc.UniqueSampleCount;
	NodeDesc.Offset = 0;

	if (NodeDesc.Count > 0)
	{
		InterlockedAdd(AllocationNodeCount, NodeDesc.Count, NodeDesc.Offset);
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupIndex == 0 && AllocationNodeCount > 0)
	{
		InterlockedAdd(OutCompactNodeCounter[0], AllocationNodeCount, AllocationNodeOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	// Allocate node space
	float PixelCoverage = 0;
	if (NodeDesc.Count > 0)
	{
		NodeDesc.Offset += AllocationNodeOffset;

		// Store final sort node data 
		if (NodeDesc.Offset + NodeDesc.Count < MaxNodeCount)
		{
			for (uint OutIndex = 0; OutIndex < NodeDesc.Count; ++OutIndex)
			{
				// VisibilityData.Coverage8bit is a weight normalising to 1 the contribution of all the compacted samples. Because later it is weighted by Categorization.PixelCoverage.
				FHairVis OutNodeVis;
				OutNodeVis.ControlPointId	= UnpackHairVisControlPointId(Samples[OutIndex].x);
				OutNodeVis.MaterialId	= UnpackHairVisMaterialId(Samples[OutIndex].x);
				OutNodeVis.Depth		= asfloat(Samples[OutIndex].w);
				OutNodeVis.Coverage8bit = To8bitCoverage(Samples[OutIndex].y / float(SampleDesc.ValidSampleCount)); // 0xff;
				const uint StoreIndex = NodeDesc.Offset + OutIndex;
				OutCompactNodeVis[StoreIndex] = PackHairVis(OutNodeVis);
				OutCompactNodeCoord[StoreIndex] = PixelCoord;
			}

			// skips separate count to transmittance pass  
			float LogViewTransmittance = -(HairCountTexture.Load(uint3(PixelCoord, 0)) / 1000.0f);
			float ViewTransmittance = pow(2.71828f, LogViewTransmittance);
			float CoverageThreshold = 0.98f;
			PixelCoverage = TransmittanceToCoverage(ViewTransmittance, CoverageThreshold);
		}
	}

	OutCompactNodeIndex[PixelCoord] = EncodeNodeDesc(NodeDesc);
	OutCoverageTexture[PixelCoord] = PixelCoverage;
}
