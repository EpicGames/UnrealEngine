// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush" 

#define MAX_SAMPLE_COUNT 8

int2 Resolution;
Texture2D<uint> NodeIndexAndOffset;
StructuredBuffer<FPackedHairSample> InNodeDataBuffer;
RWStructuredBuffer<FPackedHairSample> OutNodeDataBuffer;

// TODO: Add permutation for group 32/64
[numthreads(8, 4, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord.xy >= uint2(Resolution)))
		return;

	// All sample belonging to a pixel are already sorted by depth (closer to further)
	const FNodeDesc NodeDesc = DecodeNodeDesc(NodeIndexAndOffset.Load(uint3(PixelCoord, 0)));

	float ValidPixelSampleTotalCoverage = 0.0f;
	float SortedCoverage[MAX_SAMPLE_COUNT];
	float TotalSortedTransmittance = 1.0f;
	for (uint i = 0; i < NodeDesc.Count; ++i)
	{
		const FPackedHairSample PackedNodeData = InNodeDataBuffer[NodeDesc.Offset + i];
		const uint Coverage8bit = GetPackedHairSampleCoverage8bit(PackedNodeData);
		const float Coverage = From8bitCoverage(Coverage8bit);

		// Update current node coverage as a function of previous nodes coverage
		SortedCoverage[i] = TotalSortedTransmittance * Coverage;

		// Update transmittance for the next strands
		TotalSortedTransmittance *= 1.0f - Coverage;

		// Accumulate total coverage.
		ValidPixelSampleTotalCoverage += SortedCoverage[i];
	}

	for (uint j = 0; j < NodeDesc.Count; ++j)
	{
		// Coverage8bit is a weight normalising to 1 the contribution of all the compacted samples. Because later it is weighted by Categorization.PixelCoverage.
		// Patch the coverage on the out node
		const uint PatchedCoverage8bit = To8bitCoverage(SortedCoverage[j] / float(ValidPixelSampleTotalCoverage));

		FPackedHairSample PackedNodeData = InNodeDataBuffer[NodeDesc.Offset + j];
		PatchPackedHairSampleCoverage(PackedNodeData, PatchedCoverage8bit);
		OutNodeDataBuffer[NodeDesc.Offset + j] = PackedNodeData;
	}
}