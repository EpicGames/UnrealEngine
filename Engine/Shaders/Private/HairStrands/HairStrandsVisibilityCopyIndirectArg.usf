// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"

StructuredBuffer<uint> CounterBuffer;
RWBuffer<uint> OutArgBuffer;
uint ThreadGroupSize;
uint ItemCountPerGroup;

[numthreads(1, 1, 1)]
void CopyCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint NodeCount  = CounterBuffer[0] * ItemCountPerGroup;
	const uint TotalGroup = (NodeCount + ThreadGroupSize-1) / ThreadGroupSize;
	uint3 OutArgs;
#if 0
	// Minimize unused groups
	const uint SideSize   = uint(floor(sqrt(float(TotalGroup))));
	const uint ExtraLines = (TotalGroup - SideSize * SideSize + (SideSize - 1)) / SideSize;
	OutArgs[0] = SideSize;
	OutArgs[1] = SideSize + ExtraLines;
	OutArgs[2] = 1;
#else
	// Note: Use a fixed group count width (64) for avoiding loading the indirect args (to avoid dep. memory fetch)
	// This value is in sync with DeepTransmittanceMask.usf | HairStrandsEnvironementLightingCS.usf
	if (TotalGroup < HAIR_VISIBILITY_GROUP_COUNT_WIDTH)
	{
		OutArgs[0] = TotalGroup;
		OutArgs[1] = 1;
		OutArgs[2] = 1;
	}
	else
	{
		const uint DispatchY = ceil(float(TotalGroup) / float(HAIR_VISIBILITY_GROUP_COUNT_WIDTH));
		OutArgs[0] = HAIR_VISIBILITY_GROUP_COUNT_WIDTH;
		OutArgs[1] = DispatchY;
		OutArgs[2] = 1;
	}
#endif

	WriteDispatchIndirectArgs(OutArgBuffer, 0, OutArgs);
}