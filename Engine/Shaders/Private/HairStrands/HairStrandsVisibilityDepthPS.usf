// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "../Substrate/Substrate.ush"
#include "HairStrandsVisibilityCommon.ush"

#define OUTPUT_DEPTH 0
#define OUTPUT_MATERIALDATA 1 
#define OUTPUT_LIGHTCHANNEL 2
#define OUTPUT_MATERIALDATA_LIGHTCHANNEL 3

// Manual binding of the sample data as the uniform buffer is not created at this point
Texture2D<float>					CoverageTexture;
Texture2D<uint>						HairSampleOffset;
StructuredBuffer<FPackedHairSample> HairSampleData;

#if PERMUTATION_OUTPUT_TYPE == OUTPUT_MATERIALDATA_LIGHTCHANNEL || PERMUTATION_OUTPUT_TYPE == OUTPUT_LIGHTCHANNEL
RWTexture2D<uint> OutLightChannelMaskTexture;
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FHairSampleData
{
	float DeviceZ;
	uint LightChannelMask;
	bool bCastContactShadow;
	bool bScatterSceneLight;
};

FHairSampleData ExtractSampleData(uint3 PixelCoord)
{
	FHairSampleData Out = (FHairSampleData)0; // Inverse-Z

	const FNodeDesc NodeDesc = DecodeNodeDesc(HairSampleOffset.Load(PixelCoord));
	LOOP
	for (uint SampleIt = 0; SampleIt < NodeDesc.Count; SampleIt++)
	{
		const uint SampleIndex = NodeDesc.Offset + SampleIt;
		const FHairSample Sample = UnpackHairSample(HairSampleData[SampleIndex]);
		Out.DeviceZ = max(Out.DeviceZ, Sample.Depth); // Inverse-Z
		Out.LightChannelMask |= Sample.LightChannelMask;
		Out.bCastContactShadow = Out.bCastContactShadow || HasHairFlags(Sample.Flags, HAIR_FLAGS_CAST_CONTACT_SHADOW);
		Out.bScatterSceneLight = Out.bScatterSceneLight || HasHairFlags(Sample.Flags, HAIR_FLAGS_SCATTER_SCENE_LIGHT);
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Gbuffer patching (+ LightChannel mask)
#if SUBSTRATE_GBUFFER_FORMAT==1
#define OUTPUT_FORMAT uint 
#else
#define OUTPUT_FORMAT float4
#endif
#if PERMUTATION_OUTPUT_TYPE == OUTPUT_MATERIALDATA || PERMUTATION_OUTPUT_TYPE == OUTPUT_MATERIALDATA_LIGHTCHANNEL
void MainPS(
	in FScreenVertexOutput Input
	, out float SvDepth			: SV_DEPTH
	, out OUTPUT_FORMAT Output0	: SV_Target0
	, out OUTPUT_FORMAT Output1	: SV_Target1
	, out float4 OutColor		: SV_Target2
#if SUBSTRATE_GBUFFER_FORMAT==0
	, out OUTPUT_FORMAT Output2	: SV_Target3
#endif
)
{
	const uint3 PixelCoord = uint3(Input.Position.xy, 0);
	const float Coverage = CoverageTexture.Load(PixelCoord);

	#if PERMUTATION_OUTPUT_TYPE == OUTPUT_MATERIALDATA
	if (Coverage < 1.0f)
	{
		discard;
	}
	#endif

	// Add a depth bias in order to avoid precision issue during the composition pass, which would reject certain samples.
	// This depth is only used for screen passes like AO/SSGI/... so no need to be very accurate
	const float DepthBias = 0.001f; 
	const FHairSampleData SampleData = ExtractSampleData(PixelCoord);
	SvDepth = SampleData.DeviceZ - DepthBias;
#if SUBSTRATE_GBUFFER_FORMAT==1
	Output0 = 0; // MaterialData
	Output1 = 0; // TopLayerData

	FSubstratePixelHeader DummyHeader = InitialiseSubstratePixelHeader();
	// Set the material mode to hair for GBuffer/Material inspection
	DummyHeader.SetMaterialMode(HEADER_MATERIALMODE_HAIR);
	// Force PerObjectData contact shadow to respect hair settings, in order to have correct screen cast later
	DummyHeader.SetCastContactShadow(SampleData.bCastContactShadow ? 1 : 0);
	Output0 = DummyHeader.State;

#else
	Output0 = float4(0, 0, 0, 0);
	Output1 = 0;

	// Force PerObjectData contact shadow to respect hair settings, in order to have correct screen cast later
	Output2 = float4(0,0,0,SampleData.bCastContactShadow ? 1 : 0);
#endif
	OutColor = 0;

	#if PERMUTATION_OUTPUT_TYPE == OUTPUT_MATERIALDATA_LIGHTCHANNEL
	OutLightChannelMaskTexture[PixelCoord.xy] = SampleData.LightChannelMask;
	if (Coverage < 1.0f)
	{
		discard;
	}
	#endif

	// Do not override GBuffer data for pixel containing hair sample which will gather light from scene color. Otherwise the pulled value will be black
	if (SampleData.bScatterSceneLight)
	{
		discard;
	}
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// LightChannel mask only
#if PERMUTATION_OUTPUT_TYPE == OUTPUT_LIGHTCHANNEL
void MainPS(
	in FScreenVertexOutput Input)
{
	const uint3 PixelCoord = uint3(Input.Position.xy, 0);
	OutLightChannelMaskTexture[PixelCoord.xy] = ExtractSampleData(PixelCoord).LightChannelMask;
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Depth only
#if PERMUTATION_OUTPUT_TYPE == OUTPUT_DEPTH
uint bClear;
void MainPS(
	in FScreenVertexOutput Input,
	out float SvDepth : SV_DEPTH)
{
	SvDepth = 0; // Inverse-Z
	if (bClear == 0)
	{
		SvDepth = ExtractSampleData(uint3(Input.Position.xy, 0)).DeviceZ;
	}
}
#endif