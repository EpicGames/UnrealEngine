// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"

Texture2D<float> SceneDepthTexture;

void MainPS(
	in FScreenVertexOutput Input,
	out float OutDepth: SV_DEPTH)
{
	const uint2 PixelCoord = uint2(floor(Input.Position.xy));
	OutDepth = SceneDepthTexture.Load(uint3(PixelCoord,0));
}
