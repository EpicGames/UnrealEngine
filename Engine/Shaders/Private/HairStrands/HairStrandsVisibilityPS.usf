// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

//#define SceneTexturesStruct DeepShadowPass.SceneTextures

#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "../VelocityCommon.ush"

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#ifndef HAIR_RENDER_MODE
#error Undefined hair render mode
#endif

#define SUPPORT_OPACITY_MASK 0

#if HAIR_RENDER_MODE == RENDER_MODE_PPLL
#define SUPPORT_MATERIAL_PROPERTY 1
#endif

#if HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE || HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE_AND_HAIRCOUNT
#define SUPPORT_MATERIAL_PROPERTY 0
#endif

#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
#define SUPPORT_MATERIAL_PROPERTY 0
#endif

uint HairVisibilityPass_HairMacroGroupIndex;
uint HairVisibilityPass_HairMaterialId;
uint HairVisibilityPass_LightChannelMask;
float HairVisibilityPass_HairCoverageScale;

#if HAIR_RENDER_MODE == RENDER_MODE_PPLL
EARLYDEPTHSTENCIL	// Required for UAV operations to not happen.
#endif
void MainVisibility(
#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
	nointerpolation uint HairControlPointId : HAIR_PRIMITIVE_ID 
#else
	centroid FVertexFactoryInterpolantsVSToPS Interpolants
#endif
	, in float4 SvPosition	: SV_Position
#if HAIR_RENDER_MODE == RENDER_MODE_PPLL || HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE || HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE_AND_HAIRCOUNT
	, centroid in float WorldStrandRadius : TEXCOORD8
#endif
	  OPTIONAL_IsFrontFace
#if HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE
	, out float OutColor0 : SV_Target0
#elif HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE_AND_HAIRCOUNT
	, out float OutColor0 : SV_Target0
	, out float2 OutColor1 : SV_Target1
#elif HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
	, out uint OutColor0 : SV_Target0
#endif
)
{
	ResolvedView = ResolveView();

	#if SUPPORT_OPACITY_MASK
	{
		FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
		FPixelMaterialInputs PixelMaterialInputs;
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
		clip(GetMaterialMask(PixelMaterialInputs));
	}
	#endif

#if HAIR_RENDER_MODE != RENDER_MODE_MSAA_VISIBILITY
	float Coverage = 1;
	{
		bool bUseStableRasterization = false;
	#if HAIR_STRAND_MESH_FACTORY
		bUseStableRasterization = UseStableRasterization();
	#endif

		FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits, bUseStableRasterization);
		const float SceneDepth = ConvertFromDeviceZ(SvPosition.z); // Linear depth in world unit
		const float PixelRadius = ConvertGivenDepthRadiusForProjectionType(HairRenderInfo.RadiusAtDepth1Primary, SceneDepth); // Not correct but the coverage is not used (we count instead the number of sub-sample covered)
		const float StrandRealRadius = WorldStrandRadius;
		Coverage = saturate(StrandRealRadius / max(StrandRealRadius, PixelRadius) * HairVisibilityPass_HairCoverageScale);
	}
#endif

#if HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE
	OutColor0 = saturate(1.0f - Coverage);
#elif HAIR_RENDER_MODE == RENDER_MODE_TRANSMITTANCE_AND_HAIRCOUNT
	OutColor0 = saturate(1.0f - Coverage);
	OutColor1 = float2(Coverage, 1);
#endif

#if HAIR_RENDER_MODE == RENDER_MODE_PPLL && HAIR_STRAND_MESH_FACTORY
	const uint HairControlPointId = Interpolants.HairControlPointId;
	const uint2 PixelCoord = SvPosition.xy;
	const float2 NDC = SvPosition.xy / SvPosition.w;
	const float2 ScreenUV = NDC * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;

	// Allocate space for a new node
	uint NodeIndex;
	InterlockedAdd(HairVisibilityPass.PPLLCounter[uint2(0, 0)], 1, NodeIndex);

	if (NodeIndex < HairVisibilityPass.MaxPPLLNodeCount)
	{
		// If we can append new node, add it and make it point to the node previously in head of list
		uint NextNodeIndex;
		InterlockedExchange(HairVisibilityPass.PPLLNodeIndex[PixelCoord], NodeIndex, NextNodeIndex);
		const FPackedHairVisPPLL Node = PackHairVisPPLL(
			SvPosition.z, 
			Coverage, 
			HairControlPointId,
			HairVisibilityPass_HairMaterialId,
			NextNodeIndex);
		HairVisibilityPass.PPLLNodeData[NodeIndex] = Node;
	}
#endif

#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
  #if HAIR_STRAND_MESH_FACTORY
	OutColor0 = PackHairVisControlPointMaterialId(HairControlPointId, HairVisibilityPass_HairMaterialId);
  #else
	OutColor0 = 0;
  #endif
#endif
}


