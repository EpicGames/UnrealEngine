// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1

#ifndef PERMUTATION_CULLING
#define PERMUTATION_CULLING 0
#endif

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

RWBuffer<uint>		OutPrimIDs;
RWByteAddressBuffer OutDrawIndexedArgs;
RWByteAddressBuffer OutSWRasterPrimitiveCount;
uint				ControlPointCount;
uint				PrimIDsBufferSize;
uint				NumWorkGroups;

bool ShouldSoftwareRasterize(float3 PositionOffset, uint PrimID, out bool bSegValid)
{
	FHairControlPoint CP0 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		PrimID,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	bSegValid = CP0.Type != HAIR_CONTROLPOINT_END;

	if (!bSegValid)
	{
		return false;
	}

	FHairControlPoint CP1 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		PrimID + 1,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	// Transform to world space
	const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	float4 SP0 = mul(float4(CP0WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
	float4 SP1 = mul(float4(CP1WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));

	return (PrimID & 1) == 0; // TODO
}

[numthreads(1024, 1, 1)]
void CSMain(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	// Write IndexCountPerInstance
	if (DispatchThreadID == 0)
	{
		OutDrawIndexedArgs.Store(0, 6);
	}

#if PERMUTATION_CULLING	
	uint NumVerticesAfterCulling = ControlPointCount;
	if (HairStrandsVF_bCullingEnable)
	{
		NumVerticesAfterCulling = HairStrandsVF_CullingIndirectBuffer[3];
	}
	const uint NumBatches = (NumVerticesAfterCulling + 1023) / 1024;
#else
	const uint NumBatches = (ControlPointCount + 1023) / 1024;
#endif // PERMUTATION_CULLING
	const uint NumLoops = (NumBatches + (NumWorkGroups - 1)) / NumWorkGroups;
	const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

	LOOP for (uint LoopIndex = 0; LoopIndex < NumLoops; ++LoopIndex)
	{
		const uint BatchIndex = LoopIndex + (GroupID * NumLoops);

#if PERMUTATION_CULLING	
		uint PrimID = BatchIndex * 1024 + GroupThreadID;
		bool bSegValid = (BatchIndex < NumBatches) && (PrimID < NumVerticesAfterCulling);

		if (bSegValid && HairStrandsVF_bCullingEnable)
		{
			const uint FetchIndex0 = PrimID;
			const uint FetchIndex1 = min(FetchIndex0 + 1, NumVerticesAfterCulling - 1);

			const uint VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
			const uint VertexIndex1 = HairStrandsVF_CullingIndexBuffer[FetchIndex1];

			if (VertexIndex1 != (VertexIndex0 + 1))
			{
				bSegValid = false;
			}
			else
			{
				PrimID = VertexIndex0;
			}
		}
#else
		const uint PrimID = BatchIndex * 1024 + GroupThreadID;
		bool bSegValid = (BatchIndex < NumBatches) && (PrimID < ControlPointCount);
#endif // PERMUTATION_CULLING
		
		bool bShouldSoftwareRasterize = false;
		if (bSegValid)
		{
			bShouldSoftwareRasterize = ShouldSoftwareRasterize(PositionOffset, PrimID, bSegValid);
		}

#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		const uint NumSWSegs = WaveActiveCountBits(bSegValid && bShouldSoftwareRasterize);
		const uint NumHWSegs = WaveActiveCountBits(bSegValid && !bShouldSoftwareRasterize);
		
		uint WriteOffsetSW = 0;
		uint WriteOffsetHW = 0;
		if (WaveIsFirstLane())
		{
			if (NumSWSegs > 0)
			{
				OutSWRasterPrimitiveCount.InterlockedAdd(0, NumSWSegs, WriteOffsetSW);
			}
			if (NumHWSegs > 0)
			{
				// Increment InstanceCount
				OutDrawIndexedArgs.InterlockedAdd(1 << 2, NumHWSegs, WriteOffsetHW);
			}
		}

		WriteOffsetSW = WaveReadLaneFirst(WriteOffsetSW);
		WriteOffsetHW = WaveReadLaneFirst(WriteOffsetHW);
		
		const uint LocalWritePosSW = WavePrefixCountBits(bSegValid && bShouldSoftwareRasterize);
		const uint LocalWritePosHW = WavePrefixCountBits(bSegValid && !bShouldSoftwareRasterize);

		if (bSegValid)
		{
			const uint WritePos = bShouldSoftwareRasterize ? (WriteOffsetSW + LocalWritePosSW) : (PrimIDsBufferSize - 1 - (WriteOffsetHW + NumHWSegs) + LocalWritePosHW);
			OutPrimIDs[WritePos] = PrimID;
		}
#else
		if (bSegValid)
		{
			uint WriteOffset;
			if (bShouldSoftwareRasterize)
			{

				OutSWRasterPrimitiveCount.InterlockedAdd(0, 1, WriteOffset);
			}
			else
			{
				// Increment InstanceCount
				OutDrawIndexedArgs.InterlockedAdd(1 << 2, 1, WriteOffset);
			}
			const uint WritePos = bShouldSoftwareRasterize ? WriteOffset : (PrimIDsBufferSize - 1 - WriteOffset);
			OutPrimIDs[WritePos] = PrimID;
		}
#endif // PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	}
}