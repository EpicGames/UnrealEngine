// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1

#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsVisibilityCommon.ush"

///////////////////////////////////////////////////////////////////////////
// Common parameters

uint	TileSizeAsShift;
uint	TileSize;
float	RcpTileSize;
uint	SqrTileSize;
uint	HalfTileSize;
float	RcpHalfTileSize;
uint	SqrHalfTileSize;
int2	TileRes;

uint	NumBinners;
float	RcpNumBinners;
uint	NumRasterizers;
float	RcpNumRasterizers;

uint	MaxRasterCount;
uint	FrameIdMod8;
uint	ResolutionMultiplier;
int2	OutputResolution;
float2	OutputResolutionf;

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_BINNING || SHADER_RASTERCOMPUTE_COMPACTION || SHADER_RASTERCOMPUTE_RASTER || SHADER_RASTERCOMPUTE_RASTER_MULTI_SAMPLE || SHADER_RASTERCOMPUTE_DEPTH_GRID

///////////////////////////////////////////////////////////////////////////

/* 
// use untyped buffer for segment tiles to reduce VGPR usage - 16 bytes
struct FVisTile
{
	uint PrimOffset;
	uint PrimCount;
	uint TileCoord;
	uint MinDepth;
};
*/
#define VT_SIZE 4

#define VT_PrimOffset 0
#define VT_PrimCount 1
#define VT_Coord 2
#define VT_MinWriteIndex 3

uint PackVisTileCoord(uint2 Coord)
{
	return uint(((Coord.x & 0xff) << 0) | (((Coord.y) & 0xff) << 8));
}

uint2 UnpackVisTileCoord(uint Packed)
{
	return uint2(((Packed >> 0) & 0xff), ((Packed >> 8) & 0xff));
}

///////////////////////////////////////////////////////////////////////////

uint			MacroGroupId;
uint			HairMaterialId;


Texture2D<float> SceneDepthTexture;

uint 			VertexCount;
float			CoverageScale;

float3 NDCToPixelCoord(float4 InDC)
{
	const float3 NDC = InDC.xyz / InDC.w;
	float2 UV = NDC.xy * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	return float3(UV * OutputResolution, NDC.z);
}

void CalcHomogenousPos(in uint InPointIndex, in float3 PBO, out float4 HP, out uint Type)
{
	const FHairControlPoint CP = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		InPointIndex,
		PBO,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	const float3 WP = mul(float4(CP.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	HP = mul(float4(WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
	Type = CP.Type;
}

void CalcHomogenousPosAndRad(in uint InPointIndex, in float3 PBO, out float4 HP, out float Rad, out uint Type)
{
	const FHairControlPoint CP = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		InPointIndex,
		PBO,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	const float3 WP = mul(float4(CP.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	HP = mul(float4(WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
	Rad = CP.WorldRadius * 2000.0; // OutputResolutionf.x; //TODO: figure this out correctly?
	Type = CP.Type;
}

float ComputeLerpAlpha(int2 Coord, float2 P0, float2 P1, float SegmentLenSqRcp)
{
	// Project P onto line segment and compute the lerp alpha between P0 and P1
	// Simplification of:
	// A = P - P0
	// B = P1 - P0
	// Alpha = dot(A, B) / dot(B, B)
	const float2 P = Coord + 0.5f;
	const float Alpha = saturate(dot(P - P0, P1 - P0) * SegmentLenSqRcp);
	return Alpha;
}

float ComputePerspectiveCorrectRadius(float Rad0, float Rad1, float Alpha, float RcpW0, float RcpW1)
{
	// Alpha value for perspective correct interpolation. We store the reciprocal of w in the w component of P0 and P1,
	// so this is a simplification of:
	// (Alpha / w1) / ((1 - Alpha) / w0 + Alpha / w1)
	const float LerpedRcpW = lerp(RcpW0, RcpW1, Alpha);
	const float PerspectiveAlpha = (Alpha * RcpW1) / LerpedRcpW;
	// Divide by W to make thickness dependent on screen space depth? This division was kept from the previous line rasterization algorithm.
	const float Rad = lerp(Rad0, Rad1, PerspectiveAlpha) * LerpedRcpW;
	return Rad;
}

// Line clipping based on "CLIPPING USING HOMOGENEOUS COORDINATES" by Blinn et al.
bool BlinnLineClipping(inout float4 P0, inout float4 P1)
{
	float2 T = float2(0.0f, 1.0f);
	bool bIsRemoved = P0.w < 0.0f && P1.w < 0.0f; // Both points behind near plane

	bool bSign = false;

	UNROLL
	for (uint PlaneIdx = 0; PlaneIdx < 6; ++PlaneIdx)
	{
		// Compute boundary coordinates of both points (w+x, w-x, w+y, w-y, z, w-z)
		bSign = !bSign;
		const uint CompIdx = PlaneIdx / 2;
		const float Sign = bSign ? 1.0f : -1.0f;
		const float WFactor = PlaneIdx != 4 ? 1.0f : 0.0f;
		const float2 BC = WFactor * float2(P0.w, P1.w) + Sign * float2(P0[CompIdx], P1[CompIdx]);

		float Num = BC.x;
		float Denom = BC.x - BC.y;
		bIsRemoved = bIsRemoved || (BC.x < 0.0f && BC.y < 0.0f); // Both points outside the plane
		float Alpha = Num / Denom;
		
		// If the denominator is negative, P0 has a smaller boundary coordinate than P1, so we can assume
		// that P1 is inside the plane (or bIsRemoved is true), so we need to update the alpha for P0.
		// The reverse is true if the denominator is positive.
		if (Denom < 0.0f)
		{
			T.x = max(T.x, Alpha);
		}
		else
		{
			T.y = min(T.y, Alpha);
		}
	}

	if (!bIsRemoved)
	{
		const float4 P0Clipped = lerp(P0, P1, T.x);
		const float4 P1Clipped = lerp(P0, P1, T.y);
		P0 = P0Clipped;
		P1 = P1Clipped;
	}

	return !bIsRemoved;
}

bool ClipRaySegment(float2 AABBMin, float2 AABBMax, float4 P0, float4 P1, out float2 T, out bool2 bClipped)
{
	bClipped = false;
	T = float2(0.0f, 1.0f);
	const bool bP0Outside = any(P0.xy < AABBMin) || any(P0.xy > AABBMax);
	const bool bP1Outside = any(P1.xy < AABBMin) || any(P1.xy > AABBMax);
	if (!bP0Outside && !bP1Outside)
	{
		return true;
	}

	const float2 Origin = P0.xy;
	const float2 Dir = P1.xy - P0.xy;
	const float2 RcpDir = 1.0f / Dir;

	const float2 T0 = (AABBMin - Origin) * RcpDir;
	const float2 T1 = (AABBMax - Origin) * RcpDir;

	T.x = max(min(T0.x, T1.x), min(T0.y, T1.y));
	T.y = min(max(T0.x, T1.x), max(T0.y, T1.y));

	// Ray intersects the AABB but the segment is completely outside or no intersection at all.
	if (T.y < 0.0f || T.x > T.y)
	{
		bClipped = true;
		return false;
	}

	if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
	{
		bClipped.x = true;
	}
	if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
	{
		bClipped.y = true;
	}

	return true;
}

bool ClipRaySegment(float2 AABBMin, float2 AABBMax, inout float4 P0, inout float4 P1, inout float Rad0, inout float Rad1, out bool2 bClipped)
{
	float2 T;
	bool bIsValid = ClipRaySegment(AABBMin, AABBMax, P0, P1, T, bClipped);
	
	if (bIsValid)
	{
		const bool bP0Outside = any(P0.xy < AABBMin) || any(P0.xy > AABBMax);
		const bool bP1Outside = any(P1.xy < AABBMin) || any(P1.xy > AABBMax);

		float4 P0New = P0;
		float4 P1New = P1;
		float Rad0New = Rad0;
		float Rad1New = Rad1;
		if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
		{
			P0New = lerp(P0, P1, T.x);
			Rad0New = lerp(Rad0, Rad1, T.x);
			bClipped.x = true;
		}
		if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
		{
			P1New = lerp(P0, P1, T.y);
			Rad1New = lerp(Rad0, Rad1, T.y);
			bClipped.y = true;
		}
		P0 = P0New;
		P1 = P1New;
		Rad0 = Rad0New;
		Rad1 = Rad1New;
	}

	return bIsValid;
}

#endif // Common rasetrizer helper function & parameters

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_DEPTH_GRID

RWTexture2D<uint>			OutVisTileDepthGrid;
RWTexture2DArray<uint>		OutDepthCovTexture;
uint						NumSamples;

groupshared uint group_FurthestDepth; // (4 bytes)

[numthreads(1024, 1, 1)]
void PrepareDepthGridCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	if (GroupThreadID == 0)
	{
		group_FurthestDepth = 0xFFFFFFFF;
	}

	GroupMemoryBarrierWithGroupSync();

	// Assumes a workgroup size of 1024 threads to cover a maximum tile size of 32x32.
	if (GroupThreadID < SqrTileSize)
	{
		uint2 Coord;

		Coord.y = (GroupThreadID + 0.5f) * RcpTileSize;
		Coord.x = GroupThreadID - (Coord.y * TileSize);

		Coord += GroupID * TileSize;

		if (all(Coord < (uint2)OutputResolution))
		{
			const float Depth = SceneDepthTexture.Load(uint3(Coord, 0));
			const uint PackedDepth = PackHairVisDepthCoverage(Depth, 1.0);

			// Compute furthest depth inside this tile
			WaveInterlockedMin(group_FurthestDepth, PackedDepth);

			// Copy scene depth to (multisampled) hair depth output texture
			for (uint SampleIdx = 0; SampleIdx < NumSamples; ++SampleIdx)
			{
				InterlockedMax(OutDepthCovTexture[uint3(Coord, SampleIdx)], PackedDepth);
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadID == 0)
	{
		OutVisTileDepthGrid[GroupID] = group_FurthestDepth;
	}
}

#endif //SHADER_RASTERCOMPUTE_DEPTH_GRID

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_BINNING

#ifndef PERMUTATION_INDIRECT_PRIM_IDS
#define PERMUTATION_INDIRECT_PRIM_IDS 0
#endif

RWTexture2DArray<uint> 				OutVisTileBinningGrid;
RWBuffer<uint>						OutVisTilePrims;
RWBuffer<uint>						OutVisTileArgs;
RWByteAddressBuffer					OutVisTileData;
Texture2D<uint>						VisTileDepthGrid;
ByteAddressBuffer					IndirectPrimIDCount;
Buffer<uint>						IndirectPrimIDs;

// TODO: Without setting a limit, we are sometimes caught in an infinite loop even though this should not happen.
#define DDA_MAX_ITERATIONS 256

struct FDDAContext
{
	float2 Coord;
	float2 DeltaDist;
	float2 Step;
	float2 SideDist;
};

FDDAContext DDACreateContext(float2 RayStart, float2 RayDir)
{
	const float2 RayDirRcp = 1.0f / RayDir;

	FDDAContext Context;
	Context.Coord = floor(RayStart);
	Context.DeltaDist = abs(RayDirRcp);
	Context.Step = sign(RayDir);
	Context.SideDist = (Context.Coord - RayStart + max(Context.Step, 0.0f)) * RayDirRcp;

	return Context;
}

void DDAAdvance(inout FDDAContext Context)
{
	if (Context.SideDist.x < Context.SideDist.y)
	{
		Context.SideDist.x += Context.DeltaDist.x;
		Context.Coord.x += Context.Step.x;
	}
	else
	{
		Context.SideDist.y += Context.DeltaDist.y;
		Context.Coord.y += Context.Step.y;
	}
}

uint LoadOutVisTileData(uint index, uint offset)
{
	return OutVisTileData.Load((((index)) * VT_SIZE * 4) + ((offset) * 4));
}

void StoreOutVisTileData(uint index, uint offset, uint value)
{
	OutVisTileData.Store((((index)) * VT_SIZE * 4) + ((offset) * 4), (value));
}

groupshared uint group_LoopNum;
groupshared uint group_VerticesNum;
groupshared uint group_BatchNum;

#define TILES_TO_ALLOCATE_MAX 1024
groupshared uint group_TilesToAllocate[TILES_TO_ALLOCATE_MAX];
groupshared uint group_TilesToAllocateCount;

// The total number of line segments (VertexCount) is divided up equally between N binners - each binner = a workgroup which loops through the designated set segments in batches of 1024
// NB there is still potential to use LDS to prevent/reduce contention when adding segments to the binning grid and this may improve perf

[numthreads(1024, 1, 1)]
void BinningCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();
	if (GroupThreadID == 0)
	{
#if PERMUTATION_INDIRECT_PRIM_IDS
		group_VerticesNum = IndirectPrimIDCount.Load(0);
#else // PERMUTATION_INDIRECT_PRIM_IDS
	#if PERMUTATION_CULLING	
		group_VerticesNum = HairStrandsVF_bCullingEnable ? HairStrandsVF_CullingIndirectBuffer[3] : VertexCount;
	#else // PERMUTATION_CULLING
		group_VerticesNum = VertexCount;
	#endif //PERMUTATION_CULLING
#endif // PERMUTATION_INDIRECT_PRIM_IDS

		group_BatchNum = (group_VerticesNum + 1023) / 1024;
		group_LoopNum = (group_BatchNum + (NumBinners - 1)) * RcpNumBinners;
	}

	GroupMemoryBarrierWithGroupSync();

	LOOP for (uint LoopIndex = 0; LoopIndex < group_LoopNum; LoopIndex++)
	{
		const uint BatchIndex = LoopIndex + (GroupID * group_LoopNum);
		bool bSegValid = (BatchIndex < group_BatchNum);

#if PERMUTATION_INDIRECT_PRIM_IDS
		uint PrimID = 0;
		const uint PrimIDIndex = BatchIndex * 1024 + GroupThreadID;
		bSegValid = bSegValid && (PrimIDIndex < group_VerticesNum);
		if (bSegValid)
		{
			PrimID = IndirectPrimIDs[PrimIDIndex];
		}
#else // PERMUTATION_INDIRECT_PRIM_IDS
	#if PERMUTATION_CULLING
		uint PrimID = BatchIndex * 1024 + GroupThreadID;
		bSegValid = bSegValid && (PrimID < group_VerticesNum);

		if (bSegValid && HairStrandsVF_bCullingEnable)
		{
			const uint FetchIndex0 = PrimID;
			const uint FetchIndex1 = min(FetchIndex0 + 1, group_VerticesNum - 1);

			const uint VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
			const uint VertexIndex1 = HairStrandsVF_CullingIndexBuffer[FetchIndex1];

			if (VertexIndex1 != VertexIndex0 + 1)
			{
				bSegValid = false;
			}
			else
			{
				PrimID = VertexIndex0;
			}
		}
	#else // PERMUTATION_CULLING
		const uint PrimID = BatchIndex * 1024 + GroupThreadID;
		bSegValid = bSegValid && (PrimID < VertexCount);
	#endif // PERMUTATION_CULLING
#endif // PERMUTATION_INDIRECT_PRIM_IDS

		const uint SegmentCountLayerIdx = GroupID; // Stores number of segments per tile per workgroup.
		const uint TmpSegmentCountLayerIdx = SegmentCountLayerIdx + NumBinners; // Also stores number of segments per tile per workgroup. Used as second counter for this two pass algorithm.
		const uint TileAllocInfoLayerIdx = SegmentCountLayerIdx + NumBinners * 2; // Stores per tile per workgroup allocation info.


		uint NearestDepth = 0;
		float2 TileCoord0F = 0.0f;
		float2 TileCoord1F = 0.0f;

		// Project segment end points and clip them to the screen
		if (bSegValid)
		{
			const float3 InstancePositionOffset = HairStrandsVF_GetHairInstancePositionOffset();
			float4 H0 = 0.0f;
			float4 H1 = 0.0f;
			uint Type = -1;
			CalcHomogenousPos(PrimID, InstancePositionOffset, H0, Type);

			bool bIsEndCV = (Type == HAIR_CONTROLPOINT_END);
			bSegValid = !bIsEndCV;

			if (bSegValid)
			{
				CalcHomogenousPos(PrimID + 1, InstancePositionOffset, H1, Type);
				
				// Do clipping in homogenous coordinates
				bSegValid = BlinnLineClipping(H0, H1);

				if (bSegValid)
				{
					float3 SP0 = NDCToPixelCoord(H0);
					float3 SP1 = NDCToPixelCoord(H1);
					SP0.xy *= RcpTileSize;
					SP1.xy *= RcpTileSize;

					// For peace of mind, make sure these are actually clamped to a valid range.
					SP0 = clamp(SP0, 0.0f, float3(TileRes, 1.0f));
					SP1 = clamp(SP1, 0.0f, float3(TileRes, 1.0f));

					NearestDepth = PackHairVisDepthCoverage(max(SP0.z, SP1.z), 1.0f);
					TileCoord0F = SP0.xy;
					TileCoord1F = SP1.xy;
				}
			}
		}

		if (GroupThreadID == 0)
		{
			group_TilesToAllocateCount = 0;
		}

		GroupMemoryBarrierWithGroupSync();
		
		// Increment per workgroup per tile counters and add tiles to be allocated
		if (bSegValid)
		{
			FDDAContext DDAContext = DDACreateContext(TileCoord0F, normalize(TileCoord1F - TileCoord0F));
			const int2 EndCoord = (int2)floor(TileCoord1F);

			for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
			{
				const int2 TileCoord = (int2)floor(DDAContext.Coord);

				BRANCH
				if (NearestDepth > VisTileDepthGrid[TileCoord])
				{
					uint OldTileSegmentCount;
					InterlockedAdd(OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)], 1, OldTileSegmentCount);

					BRANCH
					if ((OldTileSegmentCount % 1024) == 0)
					{
						uint WritePos;
						InterlockedAdd(group_TilesToAllocateCount, 1, WritePos);
						if (WritePos < TILES_TO_ALLOCATE_MAX)
						{
							group_TilesToAllocate[WritePos] = PackVisTileCoord(TileCoord);
						}
					}
				}

				if (all(TileCoord == EndCoord))
				{
					break;
				}

				DDAAdvance(DDAContext);
			}
		}

		GroupMemoryBarrierWithGroupSync();

		// Allocate tiles
		const uint TilesToAllocateCount = min(TILES_TO_ALLOCATE_MAX, group_TilesToAllocateCount);
		for (uint TileIdx = GroupThreadID; TileIdx < TilesToAllocateCount; TileIdx += 1024)
		{
			const uint PackedTileCoord = group_TilesToAllocate[TileIdx];
			const uint2 TileCoord = UnpackVisTileCoord(PackedTileCoord);

			const uint TotalNewWriteCount = OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)];
			const uint TotalOldWriteCount = OutVisTileBinningGrid[uint3(TileCoord, TmpSegmentCountLayerIdx)];

			uint NewTile;
			WaveInterlockedAddScalar_(OutVisTileArgs[0], 1, NewTile);

			StoreOutVisTileData(NewTile, VT_Coord, PackedTileCoord);
			// Round down the count to the start of the tile and later compare against this to decide which tile to write to.
			StoreOutVisTileData(NewTile, VT_MinWriteIndex, TotalNewWriteCount & ~1023u);

			const uint PrevTile = (OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)] & 0xffff);

			if (TotalOldWriteCount > 0)
			{
				StoreOutVisTileData(PrevTile, VT_PrimCount, 1024);
			}

			OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)] = (PrevTile << 16) | (NewTile & 0xffff);
		}

		GroupMemoryBarrierWithGroupSync();

		// Write PrimID to tiles
		if (bSegValid)
		{
			FDDAContext DDAContext = DDACreateContext(TileCoord0F, normalize(TileCoord1F - TileCoord0F));
			const int2 EndCoord = (int2)floor(TileCoord1F);

			for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
			{
				const int2 TileCoord = (int2)floor(DDAContext.Coord);

				BRANCH
				if (NearestDepth > VisTileDepthGrid[TileCoord])
				{
					const uint PackedTiles = OutVisTileBinningGrid[uint3(TileCoord, TileAllocInfoLayerIdx)];
					const uint CurTile = (PackedTiles & 0xffff);
					const uint PrevTile = ((PackedTiles >> 16) & 0xffff);

					// Currently we need this to get our write position, but maybe there is a cheaper way to keep track of that?
					uint OldTileSegmentCount;
					InterlockedAdd(OutVisTileBinningGrid[uint3(TileCoord, TmpSegmentCountLayerIdx)], 1, OldTileSegmentCount);

					const bool bWriteToCurTile = OldTileSegmentCount >= LoadOutVisTileData(CurTile, VT_MinWriteIndex);
					const uint LocalWritePos = OldTileSegmentCount % 1024;
					const uint WritePos = (bWriteToCurTile ? CurTile : PrevTile) * 1024 + LocalWritePos;

					OutVisTilePrims[WritePos] = PrimID;

					BRANCH
					if (bWriteToCurTile)
					{
						if ((OldTileSegmentCount + 1) == OutVisTileBinningGrid[uint3(TileCoord, SegmentCountLayerIdx)])
						{
							StoreOutVisTileData(CurTile, VT_PrimCount, (OldTileSegmentCount == 1023) ? 1024 : ((OldTileSegmentCount + 1) % 1024));
						}
					}
				}

				if (all(TileCoord == EndCoord))
				{
					break;
				}

				DDAAdvance(DDAContext);
			}
		}
	}
}
#endif //SHADER_RASTERCOMPUTE_BINNING

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_COMPACTION

ByteAddressBuffer					VisTileData;
Buffer<uint> 						VisTilePrims;
Buffer<uint> 						VisTileArgs;
RWByteAddressBuffer					OutCompactedVisTileData;
RWBuffer<uint> 						OutCompactedVisTilePrims;
RWBuffer<uint> 						OutCompactedVisTileArgs;

uint LoadVisTileData(uint index, uint offset)
{
	return VisTileData.Load((((index)) * VT_SIZE * 4) + ((offset) * 4));
}

void StoreCompactedVisTileData(uint index, uint offset, uint value)
{
	OutCompactedVisTileData.Store((((index)) * VT_SIZE * 4) + ((offset) * 4), (value));
}

groupshared uint group_TotalPrimCount;
groupshared uint group_PrimWriteOffset;
groupshared uint group_NumTiles;
groupshared uint group_TilesToCompact[1024];
groupshared uint group_MaxLDSTileIdx;

[numthreads(1024, 1, 1)]
void CompactionCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	if (GroupThreadID == 0)
	{
		group_TotalPrimCount = 0;
		group_NumTiles = 0;
		group_MaxLDSTileIdx = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	const uint NumTiles = VisTileArgs[0];
	const uint PackedCoord = PackVisTileCoord(GroupID);

	// Compute total number of primitives at this tile coordinate
	uint LocalPrimCount = 0;
	{
		for (uint TileIdx = GroupThreadID; TileIdx < NumTiles; TileIdx += 1024)
		{
			const uint TilePackedCoord = LoadVisTileData(TileIdx, VT_Coord);
			if (PackedCoord == TilePackedCoord)
			{
				LocalPrimCount += LoadVisTileData(TileIdx, VT_PrimCount);
				
				uint WritePos;
				WaveInterlockedAddScalar_(group_NumTiles, 1, WritePos);
				if (WritePos < 1024)
				{
					group_TilesToCompact[WritePos] = TileIdx;
					WaveInterlockedMax(group_MaxLDSTileIdx, TileIdx);
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (LocalPrimCount > 0)
	{
		WaveInterlockedAdd(group_TotalPrimCount, LocalPrimCount);
	}

	GroupMemoryBarrierWithGroupSync();

	const uint TotalPrimCount = group_TotalPrimCount;

	if (TotalPrimCount == 0)
	{
		return;
	}

	// Allocate space
	if (GroupThreadID == 0)
	{
		const uint NumTilesToAllocate = (TotalPrimCount + 1023) / 1024;

		uint FirstCompactedTile;
		InterlockedAdd(OutCompactedVisTileArgs[0], NumTilesToAllocate, FirstCompactedTile);

		group_PrimWriteOffset = FirstCompactedTile * 1024;

		// Initialize new tiles
		for (uint TileIdx = 0; TileIdx < NumTilesToAllocate; ++TileIdx)
		{
			const uint CompactedTile = FirstCompactedTile + TileIdx;

			const uint PrimCount = min(TotalPrimCount - TileIdx * 1024, 1024);
			StoreCompactedVisTileData(CompactedTile, VT_PrimCount, PrimCount);
			StoreCompactedVisTileData(CompactedTile, VT_Coord, PackedCoord);
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Copy PrimIDs to compacted memory
	{
		uint CurrentWriteOffset = group_PrimWriteOffset;

		// First process the LDS list of tiles
		const uint NumInputTiles = min(group_NumTiles, 1024);
		for (uint LDSIdx = 0; LDSIdx < NumInputTiles; ++LDSIdx)
		{
			const uint TileIdx = group_TilesToCompact[LDSIdx];

			const uint TilePrimOffset = TileIdx * 1024;
			const uint TilePrimCount = LoadVisTileData(TileIdx, VT_PrimCount);

			if (GroupThreadID < TilePrimCount)
			{
				OutCompactedVisTilePrims[CurrentWriteOffset + GroupThreadID] = VisTilePrims[TilePrimOffset + GroupThreadID];
			}

			CurrentWriteOffset += TilePrimCount;
		}

		// Check any remaning tiles
		if (group_NumTiles > 1024)
		{
			for (uint TileIdx = group_MaxLDSTileIdx + 1; TileIdx < NumTiles; ++TileIdx)
			{
				const uint TilePackedCoord = LoadVisTileData(TileIdx, VT_Coord);
				if (PackedCoord == TilePackedCoord)
				{
					const uint TilePrimOffset = TileIdx * 1024;
					const uint TilePrimCount = LoadVisTileData(TileIdx, VT_PrimCount);

					if (GroupThreadID < TilePrimCount)
					{
						OutCompactedVisTilePrims[CurrentWriteOffset + GroupThreadID] = VisTilePrims[TilePrimOffset + GroupThreadID];
					}

					CurrentWriteOffset += TilePrimCount;
				}
			}
		}
	}
}

#endif // SHADER_RASTERCOMPUTE_COMPACTION

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_RASTER

// Wave size
#if PERMUTATION_GROUP_SIZE == 64
#define WAVE_SIZE 32
#elif PERMUTATION_GROUP_SIZE == 32
#define WAVE_SIZE 32
#else
#error Unknown group size	
#endif

// Simple rasterization algorithm that lerps between line endpoints. Is currently more robust than the Wu algorithm
// and optionally supports anti-aliasing similar to the Wu algorithm.
#define RASTER_LINEAR 0 
// Implementation of Wu's line rasterization algorithm. Currently this implementation has tile shaped artifacts when the line segment is
// clipped against the tile which is why we use the simple linear algorithm at the moment.
#define RASTER_WU 1
 // Set to 1 to enable writing to two pixels straddling the line segment when using the linear rasterization algorithm.
#define ENABLE_RASTER_LINEAR_AA 0

#define RASTER_ALGO RASTER_LINEAR

Buffer<uint> 				VisTilePrims;
Buffer<uint> 				VisTileArgs;
ByteAddressBuffer			VisTileData;
RWTexture2DArray<uint>		OutHairCountTexture;
RWTexture2DArray<uint>		OutDepthCovTexture;
RWTexture2DArray<uint>		OutPrimMatTexture;

uint LoadVisTileData(uint index, uint offset)
{
	return VisTileData.Load((((index)) * VT_SIZE * 4) + ((offset) * 4));
}

groupshared uint4 group_SubTile[1024]; //(32 x 32 x 4 x 4 bytes = 16k bytes)

groupshared float3 group_PositionOffset;
groupshared float group_ooTileLODScale;

groupshared uint group_LoopNum;
groupshared uint group_TileNum;

groupshared uint group_ThreadsPerSeg;

#define GS_SEGS 320 //this number is limited by group shared memory

groupshared float4 group_SP0[GS_SEGS];
groupshared float4 group_SP1[GS_SEGS];
groupshared float group_Rad0[GS_SEGS];
groupshared float group_Rad1[GS_SEGS];
groupshared uint group_PrimMatID[GS_SEGS];

groupshared uint group_TileIndex;

void PlotInternal(int2 Coords, float AntiAliasingFactor, float4 P0, float4 P1, float Rad0, float Rad1, float SegmentLenSqRcp, uint PackedTileMin, uint PrimMatID)
{
	const int2 IntraTileCoord = Coords - int2(((PackedTileMin >> 0) & 0xffff), ((PackedTileMin >> 16) & 0xffff));

	if (all(IntraTileCoord >= 0) && all(IntraTileCoord < int2(TileSize,TileSize)))
	{
		const float Alpha = ComputeLerpAlpha(Coords, P0.xy, P1.xy, SegmentLenSqRcp);
		const float Depth = lerp(P0.z, P1.z, Alpha);
		const uint PackedDepthCov = PackHairVisDepthCoverage(Depth, 1.0f);
		const uint LinearIndex = IntraTileCoord.x + IntraTileCoord.y * TileSize;

		// Write Depth + PrimMatID if depth test against hair depths is passed
		uint OldValue;
		InterlockedMax(group_SubTile[LinearIndex].x, PackedDepthCov, OldValue);
		if (PackedDepthCov > OldValue)
		{
			group_SubTile[LinearIndex].y = PrimMatID;
		}

		// Add hair count if depth test against scene depth is passed
		if (PackedDepthCov > group_SubTile[LinearIndex].w)
		{
			const float Rad = ComputePerspectiveCorrectRadius(Rad0, Rad1, Alpha, P0.w, P1.w);
			InterlockedAdd(group_SubTile[LinearIndex].z, min(Rad, 0.5f) * 2.0f * 1000.0f * CoverageScale * AntiAliasingFactor);
		}
	}
}

void Plot(int2 Coord, float FracY, float AntiAliasingFactor, bool bIsSteep, float4 P0, float4 P1, float Rad0, float Rad1, float SegmentLenSqRcp, uint PackedTileMin, uint PrimMatID)
{
	// First pixel
	{
		float AAFactor = AntiAliasingFactor * (1.0f - FracY);

		PlotInternal(bIsSteep ? Coord.yx : Coord.xy, AAFactor, P0, P1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
	}

	// Second pixel
	{
		float AAFactor = AntiAliasingFactor * FracY;
		Coord.y += 1;

		PlotInternal(bIsSteep ? Coord.yx : Coord.xy, AAFactor, P0, P1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
	}
}

[numthreads(1024, 1, 1)]
void RasterCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	if (GroupThreadID == 0)
	{
		group_TileNum = VisTileArgs[0];
		group_LoopNum = (float(group_TileNum) + float(NumRasterizers - 1)) * RcpNumRasterizers;

		group_PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

		/* no longer in use - keep for ref? Moving these values to group shared memory did seem to reduce VGPRs - more experimentation needed
		group_RadScale = (((HairStrandsVF_TipScale - HairStrandsVF_RootScale) * HairStrandsVF_Radius * OutputResolutionf.x) / 63.0) / 255.0;
		group_RadOffset = (HairStrandsVF_RootScale * HairStrandsVF_Radius * OutputResolutionf.x)/63.0;
		*/
	}

	GroupMemoryBarrierWithGroupSync();

	LOOP
	for (uint LoopIndex = 0; LoopIndex < group_LoopNum; LoopIndex++)
	{
		if (GroupThreadID == 0)
		{
			group_TileIndex = LoopIndex + (GroupID * group_LoopNum);
		}
		
		GroupMemoryBarrierWithGroupSync();

		bool bTileValid = (group_TileIndex < group_TileNum);

		uint PrimOffset = group_TileIndex * 1024;
		uint PrimCount = LoadVisTileData(group_TileIndex, VT_PrimCount);

		uint PackedCoord = LoadVisTileData(group_TileIndex, VT_Coord);
		uint2 SubTileMin = UnpackVisTileCoord(PackedCoord) * TileSize;

		uint PackedTileMin = ((SubTileMin.x & 0xffff) << 0) | ((SubTileMin.y & 0xffff) << 16);

		if (GroupThreadID == 0)
		{
			group_ThreadsPerSeg = 1;

			if (PrimCount <= 512)
				group_ThreadsPerSeg = 2;
			if (PrimCount <= 341)
				group_ThreadsPerSeg = 3;
			if (PrimCount <= 256)
				group_ThreadsPerSeg = 4;
			if (PrimCount <= 204)
				group_ThreadsPerSeg = 5;
			if (PrimCount <= 170)
				group_ThreadsPerSeg = 6;
			if (PrimCount <= 146)
				group_ThreadsPerSeg = 7;
			if (PrimCount <= 128)
				group_ThreadsPerSeg = 8;
			if (PrimCount <= 64)
				group_ThreadsPerSeg = 16;
			if (PrimCount <= 32)
				group_ThreadsPerSeg = 32;
		}
		
		GroupMemoryBarrierWithGroupSync();

		bool bThreadValid = (bTileValid && (GroupThreadID < (PrimCount * group_ThreadsPerSeg)));

		uint WaveCount = ((PrimCount * group_ThreadsPerSeg) + (WAVE_SIZE - 1) ) / WAVE_SIZE;
		uint WaveThreadCount = WaveCount * WAVE_SIZE;

		bool bWaveThreadValid = (bTileValid && (GroupThreadID < WaveThreadCount));

		bool bUseGroupSPs = (bThreadValid && (GroupThreadID < (min(PrimCount, GS_SEGS) * group_ThreadsPerSeg)));

		bool bGenGroupSPs = (bThreadValid && (GroupThreadID < (min(PrimCount, GS_SEGS))));

		if (bGenGroupSPs)
		{
			uint Prim = GroupThreadID;
			uint PrimID = VisTilePrims[PrimOffset + Prim];

			group_PrimMatID[Prim] = PackHairVisControlPointMaterialId(PrimID, HairMaterialId);

			uint TypeDummy;
			CalcHomogenousPosAndRad(PrimID, group_PositionOffset, group_SP0[Prim], group_Rad0[Prim], TypeDummy);
			CalcHomogenousPosAndRad(PrimID+1, group_PositionOffset, group_SP1[Prim], group_Rad1[Prim], TypeDummy);
		}

		if (bWaveThreadValid)
		{
			for (uint LinearIndex = GroupThreadID; LinearIndex < SqrTileSize; LinearIndex += WaveThreadCount)
			{
				uint2 Coord;

				Coord.y = (float(LinearIndex) + 0.5f) * RcpTileSize;
				Coord.x = LinearIndex - (Coord.y * TileSize);

				Coord += uint2(((PackedTileMin >> 0) & 0xffff), ((PackedTileMin >> 16) & 0xffff));

				group_SubTile[LinearIndex].x = OutDepthCovTexture[uint3(Coord, 0)];
				group_SubTile[LinearIndex].y = GetInvalidHairControlPointId();
				group_SubTile[LinearIndex].z = 0;
				group_SubTile[LinearIndex].w = PackHairVisDepthCoverage(SceneDepthTexture.Load(uint3(Coord, 0)), 1.0f);
			}
		}

		GroupMemoryBarrierWithGroupSync();

		if (bThreadValid)
		{
			uint Prim = uint((float(GroupThreadID) + 0.5f) / float(group_ThreadsPerSeg));
			uint PModTPS = GroupThreadID - (Prim * group_ThreadsPerSeg);

			uint PrimMatID;
			float4 SP0;
			float4 SP1;
			float Rad0;
			float Rad1;

			if (bUseGroupSPs)
			{
				PrimMatID = group_PrimMatID[Prim];
				SP0 = group_SP0[Prim];
				SP1 = group_SP1[Prim];
				Rad0 = group_Rad0[Prim];
				Rad1 = group_Rad1[Prim];
			}
			else
			{
				uint PrimID = VisTilePrims[PrimOffset + Prim];
				PrimMatID = PackHairVisControlPointMaterialId(PrimID, HairMaterialId);

				uint TypeDummy;
				CalcHomogenousPosAndRad(PrimID, group_PositionOffset, SP0, Rad0, TypeDummy);
				CalcHomogenousPosAndRad(PrimID + 1, group_PositionOffset, SP1, Rad1, TypeDummy);
			}

			// Clipping
			{
				SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
				SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);
				
				// Clip against tile
				const float2 TileMin = float2(((PackedTileMin >> 0) & 0xffff), ((PackedTileMin >> 16) & 0xffff));
				const float2 TileMax = TileMin + TileSize;
				bool2 bClipped = false;
				ClipRaySegment(TileMin - 0.5f, TileMax + 0.5f, SP0, SP1, Rad0, Rad1, bClipped);
			}

			const float SegmentLenSqRcp = 1.0f / dot(SP1.xy - SP0.xy, SP1.xy - SP0.xy);

#if RASTER_ALGO == RASTER_LINEAR

			const bool bIsSteep = abs(SP1.x - SP0.x) < abs(SP1.y - SP0.y);
			const float X0 = bIsSteep ? min(SP0.y, SP1.y) : min(SP0.x, SP1.x);
			const float X1 = bIsSteep ? max(SP0.y, SP1.y) : max(SP0.x, SP1.x);
			const int NumSteps = (int)(ceil(X1) - floor(X0));
			const float RcpNumSteps = 1.0f / (X1 - X0);

			LOOP
			for (int J = PModTPS; J < NumSteps; J += group_ThreadsPerSeg)
			{
				const float Alpha = saturate(J * RcpNumSteps);
				const float4 SP = lerp(SP0, SP1, Alpha);

				const float AntiAliasingFactor = 1.0f;
#if !ENABLE_RASTER_LINEAR_AA
				PlotInternal(SP.xy, AntiAliasingFactor, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
#else
				const float2 Coord = (bIsSteep ? SP.yx : SP.xy) - 0.5f;
				const float FracY = frac(Coord.y);
				Plot(Coord, FracY, AntiAliasingFactor, bIsSteep, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
#endif // !ENABLE_RASTER_LINEAR_AA
			}
#elif RASTER_ALGO == RASTER_WU
			// Wu's line algorithm. Currently this has some weird artifacts when clipping to tiles.
			// TODO: Remove this entirely or fix the artifacts.
			{
				const bool bIsSteep = abs(SP1.y - SP0.y) > abs(SP1.x - SP0.x);

				if (bIsSteep)
				{
					SP0.xy = SP0.yx;
					SP1.xy = SP1.yx;
				}
				if (SP0.x > SP1.x)
				{
					float4 Tmp = SP0;
					SP0 = SP1;
					SP1 = Tmp;
				}

				const float2 D = SP1.xy - SP0.xy;
				const float Gradient = abs(D.x) < 1e-5f ? 1.0f : D.y / D.x;

				float DeltaY = 0.0f;

				// First endpoint
				int2 Px0;
				{
					const float2 SP0Int = SP0.xy - 0.5f; // transform to integer grid with pixel centers at 0.0 instead of 0.5.
					float2 End;
					End.x = floor(SP0Int.x);
					End.y = SP0Int.y + Gradient * (End.x - SP0Int.x);

					const float GapX = 1.0f;// 1.0f - frac(SP0Int.x + 0.5f);

					Px0 = int2(End.x, floor(End.y));
					
					if (PModTPS == 0)
					{
						const float FracY = frac(End.y);
						Plot(Px0, FracY, GapX, bIsSteep, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
					}

					DeltaY = End.y + Gradient; // First y-intersection for the main loop
				}

				// Second endpoint
				int2 Px1;
				{
					const float2 SP1Int = SP1.xy - 0.5f; // transform to integer grid with pixel centers at 0.0 instead of 0.5.
					float2 End;
					End.x = floor(SP1Int.x);
					End.y = SP1Int.y + Gradient * (End.x - SP1Int.x);
					const float GapX = 1.0f;// frac(SP1Int.x + 0.5f);

					Px1 = float2(End.x, floor(End.y));

					if (PModTPS == 0)
					{
						const float FracY = frac(End.y);
						Plot(Px1, FracY, GapX, bIsSteep, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
					}
				}

				// Main loop
				const int XBegin = Px0.x + 1 + PModTPS;
				const int XEnd = Px1.x;
				DeltaY += PModTPS * Gradient;
				for (int X = XBegin; X < XEnd; X += group_ThreadsPerSeg)
				{
					const int2 Coord = int2(X, floor(DeltaY));
					const float FracY = frac(DeltaY);
					Plot(Coord, FracY, 1.0f, bIsSteep, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedTileMin, PrimMatID);
					DeltaY += group_ThreadsPerSeg * Gradient;
				}
			}
#endif // RASTER_ALGO == RASTER_LINEAR
		}

		GroupMemoryBarrierWithGroupSync();

		if (bWaveThreadValid) 
		{
			for (uint LinearIndex = GroupThreadID; LinearIndex < SqrTileSize; LinearIndex += WaveThreadCount)
			{
				uint2 Coord;

				Coord.y = (float(LinearIndex) + 0.5f) * RcpTileSize;
				Coord.x = LinearIndex - (Coord.y * TileSize);

				Coord += uint2(((PackedTileMin >> 0) & 0xffff), ((PackedTileMin >> 16) & 0xffff));

				if (group_SubTile[LinearIndex].y != GetInvalidHairControlPointId())
				{
					uint oldValue;
					InterlockedMax(OutDepthCovTexture[uint3(Coord, 0)], group_SubTile[LinearIndex].x, oldValue);
					if (group_SubTile[LinearIndex].x > oldValue)
					{
						OutPrimMatTexture[uint3(Coord, 0)] = group_SubTile[LinearIndex].y;
					}
				}
				InterlockedAdd(OutHairCountTexture[uint3(Coord, 0)], group_SubTile[LinearIndex].z);
			}
		}

		GroupMemoryBarrierWithGroupSync();
	}
}

#endif //SHADER_RASTERCOMPUTE_RASTER

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_RASTER_MULTI_SAMPLE

#ifndef PERMUTATION_MULTI_SAMPLE_COUNT
#define PERMUTATION_MULTI_SAMPLE_COUNT 1
#endif

// Wave size
#if PERMUTATION_GROUP_SIZE == 64
#define WAVE_SIZE 32
#elif PERMUTATION_GROUP_SIZE == 32
#define WAVE_SIZE 32
#else
#error Unknown group size	
#endif

Buffer<uint> 				VisTilePrims;
Buffer<uint> 				VisTileArgs;
ByteAddressBuffer			VisTileData;
RWTexture2D<uint>			OutHairCountTexture;
RWTexture2DArray<uint>		OutDepthCovTexture;
RWTexture2DArray<uint>		OutPrimMatTexture;

uint LoadVisTileData(uint index, uint offset)
{
	return VisTileData.Load((((index)) * VT_SIZE * 4) + ((offset) * 4));
}

groupshared uint group_SubTileSceneDepth[256]; // (16 x 16 x 4 bytes = 1k bytes)
groupshared uint group_SubTileHairCount[256]; // (16 x 16 x 4 bytes = 1k bytes)
groupshared uint group_SubTileHairDepth[PERMUTATION_MULTI_SAMPLE_COUNT][256]; // (16 x 16 x 4 bytes = 1k bytes) per sample
groupshared uint group_SubTilePrimMatID[PERMUTATION_MULTI_SAMPLE_COUNT][256]; // (16 x 16 x 4 bytes = 1k bytes) per sample

groupshared float3 group_PositionOffset;

groupshared uint group_LoopNum;
groupshared uint group_TileNum;

float GetDistanceToLine(float2 P1, float2 P2, float2 P3, float RcpLineSegLength)
{
	// We can compute the distance of P1 to the line defined by P2 and P3 as the height of the triangle spanned by these points.
	// Area of triangle: A = 0.5 * h * b where h is the triangle height and b is the length of the base side.
	// Solving for h gives: h = (2 * A) / b
	// We can compute A using the determinant: A = 0.5 * abs(det(P1, P2, P3))
	// After some simplification, this results in the following:
	float A = abs(P1.x * (P2.y - P3.y) + P2.x * (P3.y - P1.y) + P3.x * (P1.y - P2.y));
	return A * RcpLineSegLength;
}

uint GetCoverageMask(int2 PixelCoord, float2 P0, float2 P1)
{
	const float LineThickness = 1.0f / PERMUTATION_MULTI_SAMPLE_COUNT; // In pixel units 
	uint Mask = 0;

	// Set origin to PixelCoord
	P0 -= PixelCoord;
	P1 -= PixelCoord;

	const float RcpLineSegLength = 1.0f / distance(P0, P1);

#if PERMUTATION_MULTI_SAMPLE_COUNT == 1
	Mask |= (GetDistanceToLine(float2(0.5f, 0.5f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 0u) : 0;
#elif PERMUTATION_MULTI_SAMPLE_COUNT == 2
	Mask |= (GetDistanceToLine(float2(0.75f, 0.75f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 0u) : 0;
	Mask |= (GetDistanceToLine(float2(0.25f, 0.25f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 1u) : 0;
#elif PERMUTATION_MULTI_SAMPLE_COUNT == 4
	Mask |= (GetDistanceToLine(float2(0.375f, 0.125f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 0u) : 0;
	Mask |= (GetDistanceToLine(float2(0.875f, 0.375f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 1u) : 0;
	Mask |= (GetDistanceToLine(float2(0.125f, 0.625f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 2u) : 0;
	Mask |= (GetDistanceToLine(float2(0.625f, 0.875f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 3u) : 0;
#elif PERMUTATION_MULTI_SAMPLE_COUNT == 8
	Mask |= (GetDistanceToLine(float2(0.5625f, 0.3125f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 0u) : 0;
	Mask |= (GetDistanceToLine(float2(0.4375f, 0.6875f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 1u) : 0;
	Mask |= (GetDistanceToLine(float2(0.8125f, 0.5625f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 2u) : 0;
	Mask |= (GetDistanceToLine(float2(0.3125f, 0.1875f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 3u) : 0;
	Mask |= (GetDistanceToLine(float2(0.1875f, 0.8125f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 4u) : 0;
	Mask |= (GetDistanceToLine(float2(0.0625f, 0.4375f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 5u) : 0;
	Mask |= (GetDistanceToLine(float2(0.6875f, 0.9375f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 6u) : 0;
	Mask |= (GetDistanceToLine(float2(0.9375f, 0.0625f), P0, P1, RcpLineSegLength) <= LineThickness) ? (1u << 7u) : 0;
#else
#error Unsupported PERMUTATION_MULTI_SAMPLE_COUNT! Must be 1, 2, 4 or 8!
#endif

	return Mask;
}

void Plot(int2 Coord, float4 P0, float4 P1, float Rad0, float Rad1, float SegmentLenSqRcp, uint PackedHalfTileMin, uint PrimMatID)
{
	const int2 IntraTileCoord = Coord - int2(((PackedHalfTileMin >> 0) & 0xffff), ((PackedHalfTileMin >> 16) & 0xffff));

	if (all(IntraTileCoord >= 0) && all(IntraTileCoord < int2(HalfTileSize,HalfTileSize)))
	{
		const float Alpha = ComputeLerpAlpha(Coord, P0.xy, P1.xy, SegmentLenSqRcp);
		const float Depth = lerp(P0.z, P1.z, Alpha);
		const uint PackedDepthCov = PackHairVisDepthCoverage(Depth, 1.0f);
		const uint LinearIndex = IntraTileCoord.x + IntraTileCoord.y * HalfTileSize;

		// Test against scene depth
		if (PackedDepthCov > group_SubTileSceneDepth[LinearIndex])
		{
			const float Rad = ComputePerspectiveCorrectRadius(Rad0, Rad1, Alpha, P0.w, P1.w);
			const uint HairCount = min(Rad, 0.5f) * 2.0f * 1000.0f * CoverageScale;

			const uint CoverageMask = GetCoverageMask(Coord, P0.xy, P1.xy);

			// Accumulate hair count
			if (CoverageMask)
			{
				InterlockedAdd(group_SubTileHairCount[LinearIndex], HairCount);
			}

			UNROLL for (uint SampleIdx = 0; SampleIdx < PERMUTATION_MULTI_SAMPLE_COUNT; ++SampleIdx)
			{
				if (CoverageMask & (1u << SampleIdx))
				{
					// Write Depth + PrimMatID if depth test against hair depths is passed
					uint OldValue;
					InterlockedMax(group_SubTileHairDepth[SampleIdx][LinearIndex], PackedDepthCov, OldValue);
					if (PackedDepthCov > OldValue)
					{
						group_SubTilePrimMatID[SampleIdx][LinearIndex] = PrimMatID;
					}
				}
			}
		}
	}
}

[numthreads(1024, 1, 1)]
void RasterMultiSampleCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	if (GroupThreadID == 0)
	{
		group_TileNum = VisTileArgs[0];
		group_LoopNum = (float(group_TileNum) + float(NumRasterizers - 1)) * RcpNumRasterizers;

		group_PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();
	}

	GroupMemoryBarrierWithGroupSync();

	LOOP for (uint LoopIndex = 0; LoopIndex < group_LoopNum; LoopIndex++)
	{
		const uint TileIndex = LoopIndex + (GroupID * group_LoopNum);

		if (TileIndex >= group_TileNum)
		{
			return;
		}

		const uint PrimOffset = TileIndex * 1024;
		const uint PrimCount = LoadVisTileData(TileIndex, VT_PrimCount);

		const uint PackedCoord = LoadVisTileData(TileIndex, VT_Coord);
		const uint2 TileMin = UnpackVisTileCoord(PackedCoord) * TileSize;

		const uint PackedTileMin = ((TileMin.x & 0xffff) << 0) | ((TileMin.y & 0xffff) << 16);

		uint ThreadsPerSeg = 1;

		if (PrimCount <= 512)
			ThreadsPerSeg = 2;
		if (PrimCount <= 341)
			ThreadsPerSeg = 3;
		if (PrimCount <= 256)
			ThreadsPerSeg = 4;
		if (PrimCount <= 204)
			ThreadsPerSeg = 5;
		if (PrimCount <= 170)
			ThreadsPerSeg = 6;
		if (PrimCount <= 146)
			ThreadsPerSeg = 7;
		if (PrimCount <= 128)
			ThreadsPerSeg = 8;
		if (PrimCount <= 64)
			ThreadsPerSeg = 16;
		if (PrimCount <= 32)
			ThreadsPerSeg = 32;

		const bool bThreadValid = (GroupThreadID < (PrimCount * ThreadsPerSeg));
		const uint Prim = uint((float(GroupThreadID) + 0.5f) / float(ThreadsPerSeg));
		const uint PModTPS = GroupThreadID - (Prim * ThreadsPerSeg);

		float4 SP0 = 0;
		float4 SP1 = 0;
		float Rad0 = 0;
		float Rad1 = 0;
		bool bIsEndPoint = false;
		uint PrimMatID = ~0;

		if (bThreadValid)
		{
			const uint PrimID = VisTilePrims[PrimOffset + Prim];
			PrimMatID = PackHairVisControlPointMaterialId(PrimID, HairMaterialId);

			uint Type;
			CalcHomogenousPosAndRad(PrimID, group_PositionOffset, SP0, Rad0, Type);
			CalcHomogenousPosAndRad(PrimID + 1, group_PositionOffset, SP1, Rad1, Type);
			bIsEndPoint = (Type == HAIR_CONTROLPOINT_END);

			SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
			SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);
		}

		// Split 32x32 tile into 4 16x16 tiles that are processed one after another. 
		// This is to reduce LDS memory pressure.
		UNROLL for (uint SubTileIdx = 0; SubTileIdx < 4; ++SubTileIdx)
		{
			const uint2 SubTileMin = TileMin + uint2((SubTileIdx == 0 || SubTileIdx == 2) ? 0 : HalfTileSize, SubTileIdx < 2 ? 0 : HalfTileSize);
			const uint2 SubTileMax = SubTileMin + HalfTileSize;
			const uint PackedSubTileMin = ((SubTileMin.x & 0xFFFF) << 0u) | ((SubTileMin.y & 0xFFFF) << 16u);

			// Initialize LDS
			if (GroupThreadID < SqrHalfTileSize)
			{
				uint2 Coord;
				Coord.y = (float(GroupThreadID) + 0.5f) * RcpHalfTileSize;
				Coord.x = GroupThreadID - (Coord.y * HalfTileSize);
				Coord += uint2(((PackedSubTileMin >> 0u) & 0xFFFF), ((PackedSubTileMin >> 16u) & 0xFFFF));

				group_SubTileSceneDepth[GroupThreadID] = PackHairVisDepthCoverage(SceneDepthTexture.Load(uint3(Coord, 0)), 1.0f);
				group_SubTileHairCount[GroupThreadID] = 0;

				UNROLL for (uint SampleIdx = 0; SampleIdx < PERMUTATION_MULTI_SAMPLE_COUNT; ++SampleIdx)
				{
					const uint HairDepth = OutDepthCovTexture[uint3(Coord, SampleIdx)];
					group_SubTileHairDepth[SampleIdx][GroupThreadID] = HairDepth;
					group_SubTilePrimMatID[SampleIdx][GroupThreadID] = GetInvalidHairControlPointId();
				}
			}

			GroupMemoryBarrierWithGroupSync();

			// Rasterize to LDS
			if (bThreadValid)
			{
				const uint2 SubTileMin = uint2(((PackedSubTileMin >> 0u) & 0xFFFF), ((PackedSubTileMin >> 16u) & 0xFFFF));
				const uint2 SubTileMax = SubTileMin + HalfTileSize;
				bool2 bClipped;
				float2 T;
				const bool bVisible = ClipRaySegment(SubTileMin - 0.5f, SubTileMax + 0.5f, SP0, SP1, T, bClipped);
				T = saturate(T);
				
				if (bVisible)
				{
					const float2 SP0Clipped = lerp(SP0, SP1, T.x).xy;
					const float2 SP1Clipped = lerp(SP0, SP1, T.y).xy;
					const bool bIsSteep = abs(SP1.x - SP0.x) < abs(SP1.y - SP0.y);
					const float X0 = bIsSteep ? min(SP0Clipped.y, SP1Clipped.y) : min(SP0Clipped.x, SP1Clipped.x);
					const float X1 = bIsSteep ? max(SP0Clipped.y, SP1Clipped.y) : max(SP0Clipped.x, SP1Clipped.x);
					const int NumSteps = (int)(ceil(X1) - floor(X0));
					const float RcpNumSteps = 1.0f / (X1 - X0);

					const float SegmentLenSqRcp = 1.0f / dot(SP1.xy - SP0.xy, SP1.xy - SP0.xy);

					const int End = !bClipped.y && !bIsEndPoint ? (NumSteps - 1) : NumSteps;
					LOOP for (int J = PModTPS; J < End; J += ThreadsPerSeg)
					{
						const float Alpha = lerp(T.x, T.y, saturate(J * RcpNumSteps));
						const float2 SP = lerp(SP0.xy, SP1.xy, Alpha);
						Plot(SP, SP0, SP1, Rad0, Rad1, SegmentLenSqRcp, PackedSubTileMin, PrimMatID);
					}
				}
			}

			GroupMemoryBarrierWithGroupSync();

			// Write out to global memory
			if (GroupThreadID < SqrHalfTileSize)
			{
				uint2 Coord;
				Coord.y = (float(GroupThreadID) + 0.5f) * RcpHalfTileSize;
				Coord.x = GroupThreadID - (Coord.y * HalfTileSize);
				Coord += uint2(((PackedSubTileMin >> 0u) & 0xFFFF), ((PackedSubTileMin >> 16u) & 0xFFFF));

				const uint HairCount = group_SubTileHairCount[GroupThreadID];
				if (HairCount != 0)
				{
					InterlockedAdd(OutHairCountTexture[Coord], HairCount);
				}

				UNROLL for (uint SampleIdx = 0; SampleIdx < PERMUTATION_MULTI_SAMPLE_COUNT; ++SampleIdx)
				{
					const uint3 SampleCoord = uint3(Coord, SampleIdx);
					
					const uint PrimMatID = group_SubTilePrimMatID[SampleIdx][GroupThreadID];
					if (PrimMatID != GetInvalidHairControlPointId())
					{
						const uint HairDepth = group_SubTileHairDepth[SampleIdx][GroupThreadID];

						uint OldValue;
						InterlockedMax(OutDepthCovTexture[SampleCoord], HairDepth, OldValue);
						if (HairDepth > OldValue)
						{
							OutPrimMatTexture[SampleCoord] = PrimMatID;
						}
					}
				}
			}

			GroupMemoryBarrierWithGroupSync();
		}
	}
}

#endif //SHADER_RASTERCOMPUTE_RASTER_MULTI_SAMPLE

///////////////////////////////////////////////////////////////////////////

#if SHADER_RASTERCOMPUTE_DEBUG

#include "../ShaderPrint.ush"

Texture2D<uint>						VisTileDepthGrid;
Texture2DArray<uint> 				VisTileBinningGrid;
Buffer<uint>						VisTileArgs;
uint								MacroGroupId;
uint								PrimitiveInfoIndex;
uint								TotalPrimitiveInfoCount;

#define TilePrintOffset (TileSize >> 1)

float4 Transparent(float4 Color) { return float4(Color.xyz, 0.5f); }

uint GetTileTotalSegment(uint2 TileCoord, bool bPrintDetails)
{
	const float TileDisplayScale = 1.5f;
	const uint DisplayTileSize = TileSize * TileDisplayScale;
	uint2 InlinedTileCoord = uint2(0, 0);

	uint TotalSegments = 0;
	const uint BinCount = NumBinners;// * 2; // Each binner fill in 2 bins, see binning algo.
	for (uint BinIt = 0; BinIt < BinCount; ++BinIt)
	{
		const uint CurrTileSegments = VisTileBinningGrid.Load(uint4(TileCoord, BinIt, 0));
		TotalSegments += CurrTileSegments;

		if (bPrintDetails)
		{
			AddFilledQuadSS(InlinedTileCoord * DisplayTileSize, (InlinedTileCoord + 1) * DisplayTileSize, CurrTileSegments > 0 ? Transparent(ColorLightGreen) : Transparent(ColorLightRed));
			AddQuadSS(InlinedTileCoord * DisplayTileSize, (InlinedTileCoord + 1) * DisplayTileSize, ColorYellow);

			FShaderPrintContext Context = InitShaderPrintContext(true, InlinedTileCoord * DisplayTileSize + TilePrintOffset);
			Print(Context, CurrTileSegments, FontWhite);
			++InlinedTileCoord.x;

			// Span details onto 2 lines
			if (BinIt == NumBinners-1)
			{
				InlinedTileCoord.x = 0;
				++InlinedTileCoord.y;
			}
		}
	}
	return TotalSegments;
}

void PrintTile(uint2 TileCoord, uint TotalSegments, bool bPrintText)
{
	AddFilledQuadSS(TileCoord * TileSize, (TileCoord + 1) * TileSize, TotalSegments > 0 ? Transparent(ColorLightGreen) : Transparent(ColorLightRed));
	if (bPrintText)
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, TileCoord * TileSize + uint2(0, TileSize * 1.5f));
		Print(Context, TotalSegments, FontWhite);

		AddQuadSS(TileCoord * TileSize, (TileCoord + 1) * TileSize, ColorYellow);
	}
}

[numthreads(8, 8, 1)]
void MainCS(uint3 ThreadId : SV_DispatchThreadID)
{
	// Info/Stats
	if (all(ThreadId == 0))
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 110));
		Print(Context, TEXT("Raster compute         "), FontYellow); Newline(Context);
		Print(Context, TEXT("Macro Group Id       : "), FontSilver); Print(Context, MacroGroupId, FontWhite); Newline(Context);
		Print(Context, TEXT("Primitive Info       : "), FontSilver); Print(Context, PrimitiveInfoIndex, FontWhite, 2, 0); Print(Context, TEXT("/"), FontSilver); Print(Context, TotalPrimitiveInfoCount, FontWhite, 2, 0); Newline(Context);
		Newline(Context);

		Print(Context, TEXT("Configuration          "), FontYellow); Newline(Context);
		Print(Context, TEXT("Output Resolution    : "), FontSilver); Print(Context, OutputResolution, FontWhite); Newline(Context);
		Print(Context, TEXT("Resolution Multiplier: "), FontSilver); Print(Context, ResolutionMultiplier, FontWhite); Newline(Context);
		Newline(Context);
		
		Print(Context, TEXT("Tile Size            : "), FontSilver); Print(Context, TileSize, FontWhite); Newline(Context);
		Print(Context, TEXT("Tile Res             : "), FontSilver); Print(Context, TileRes.x, FontWhite, 2, 0); Print(Context, TEXT("x"), FontSilver); Print(Context, TileRes.y, FontWhite, 2, 0); Newline(Context);
		Newline(Context);
		
		Print(Context, TEXT("Num Binners          : "), FontSilver); Print(Context, NumBinners, FontWhite); Newline(Context);
		Print(Context, TEXT("Num Rasterizers      : "), FontSilver); Print(Context, NumRasterizers, FontWhite); Newline(Context);
		Print(Context, TEXT("Max Raster Count     : "), FontSilver); Print(Context, MaxRasterCount, FontWhite); Newline(Context);
		Newline(Context);

		Print(Context, TEXT("Allocated Tile Count : "), FontSilver); Print(Context, VisTileArgs[0], FontWhite); Newline(Context);
		

	}

	// Cursor info	
	if (all(ThreadId == 0) && all(ShaderPrintData.CursorCoord >= 0))
	{
		const uint2 PixelCoord = ShaderPrintData.CursorCoord;
		const uint2 TileCoord  = PixelCoord >> TileSizeAsShift;
	
		const uint TotalSegments = GetTileTotalSegment(TileCoord, true);
		PrintTile(TileCoord, TotalSegments, true);
	}

	// All tile
	{
		const uint2 TileCoord = ThreadId.xy;
		const uint TotalSegments = GetTileTotalSegment(TileCoord, false);
		if (TotalSegments)
		{
			PrintTile(TileCoord, TotalSegments, false);
		}
	}
}
#endif //SHADER_RASTERCOMPUTE_DEBUG

