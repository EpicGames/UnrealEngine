// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1

#ifndef PERMUTATION_INDIRECT_PRIM_IDS
#define PERMUTATION_INDIRECT_PRIM_IDS 0
#endif

#ifndef PERMUTATION_CULLING
#define PERMUTATION_CULLING 0
#endif

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

RWTexture2D<uint>	OutHairCountTexture;
RWTexture2D<uint>	OutDepthCovTexture;
RWTexture2D<uint>	OutPrimMatTexture;
Texture2D<float>	SceneDepthTexture;
ByteAddressBuffer	IndirectPrimIDCount;
Buffer<uint>		IndirectPrimIDs;
float2				OutputResolution;
uint				HairMaterialId;
uint				ControlPointCount;
float				CoverageScale;
uint				NumWorkGroups;

// Line clipping based on "CLIPPING USING HOMOGENEOUS COORDINATES" by Blinn et al.
bool BlinnLineClipping(inout float4 P0, inout float4 P1)
{
	float2 T = float2(0.0f, 1.0f);
	bool bIsRemoved = P0.w < 0.0f && P1.w < 0.0f; // Both points behind near plane

	bool bSign = false;

	UNROLL for (uint PlaneIdx = 0; PlaneIdx < 6; ++PlaneIdx)
	{
		// Compute boundary coordinates of both points (w+x, w-x, w+y, w-y, z, w-z)
		bSign = !bSign;
		const uint CompIdx = PlaneIdx / 2;
		const float Sign = bSign ? 1.0f : -1.0f;
		const float WFactor = PlaneIdx != 4 ? 1.0f : 0.0f;
		const float2 BC = WFactor * float2(P0.w, P1.w) + Sign * float2(P0[CompIdx], P1[CompIdx]);

		float Num = BC.x;
		float Denom = BC.x - BC.y;
		bIsRemoved = bIsRemoved || (BC.x < 0.0f && BC.y < 0.0f); // Both points outside the plane
		float Alpha = Num / Denom;

		// If the denominator is negative, P0 has a smaller boundary coordinate than P1, so we can assume
		// that P1 is inside the plane (or bIsRemoved is true), so we need to update the alpha for P0.
		// The reverse is true if the denominator is positive.
		if (Denom < 0.0f)
		{
			T.x = max(T.x, Alpha);
		}
		else
		{
			T.y = min(T.y, Alpha);
		}
	}

	if (!bIsRemoved)
	{
		const float4 P0Clipped = lerp(P0, P1, T.x);
		const float4 P1Clipped = lerp(P0, P1, T.y);
		P0 = P0Clipped;
		P1 = P1Clipped;
	}

	return !bIsRemoved;
}

float3 NDCToPixelCoord(float4 InDC)
{
	const float3 NDC = InDC.xyz / InDC.w;
	float2 UV = NDC.xy * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	return float3(UV * OutputResolution, NDC.z);
}

void Plot(int2 Coord, float AntiAliasingFactor, float4 P0, float4 P1, float Rad0, float Rad1, float SegmentLenSqRcp, uint PrimMatID)
{
	// Project P onto line segment and compute the lerp alpha between P0 and P1
	// Simplification of:
	// A = P - P0
	// B = P1 - P0
	// Alpha = dot(A, B) / dot(B, B)
	const float2 P = Coord + 0.5f;
	const float Alpha = saturate(dot(P - P0.xy, P1.xy - P0.xy) * SegmentLenSqRcp);
	const float Depth = lerp(P0.z, P1.z, Alpha);
	const uint PackedDepthCov = PackHairVisDepthCoverage(Depth, 1.0f);

	// Write Depth + PrimMatID if depth test against hair depths is passed
	uint OldValue;
	InterlockedMax(OutDepthCovTexture[Coord], PackedDepthCov, OldValue);
	if (PackedDepthCov > OldValue)
	{
		OutPrimMatTexture[Coord] = PrimMatID;
	}

	// Add hair count if depth test against scene depth is passed
	if (PackedDepthCov > PackHairVisDepthCoverage(SceneDepthTexture.Load(uint3(Coord, 0)), 1.0f))
	{
		// Alpha value for perspective correct interpolation. We store the reciprocal of w in the w compoennt,
		// so this is a simplification of:
		// (Alpha / w1) / ((1 - Alpha) / w0 + Alpha / w1)
		const float LerpedRcpW = lerp(P0.w, P1.w, Alpha);
		const float PerspectiveAlpha = (Alpha * P1.w) / LerpedRcpW;
		// Divide by W to make thickness dependent on screen space depth? This division was kept from the previous line rasterization algorithm.
		const float Rad = lerp(Rad0, Rad1, PerspectiveAlpha) * LerpedRcpW;

		InterlockedAdd(OutHairCountTexture[Coord], min(Rad, 0.5f) * 2.0f * 1000.0f * CoverageScale * AntiAliasingFactor);
	}
}

[numthreads(64, 1, 1)]
void CSMain(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

#if PERMUTATION_INDIRECT_PRIM_IDS
	const uint NumControlPoints = IndirectPrimIDCount.Load(0);
#else // PERMUTATION_INDIRECT_PRIM_IDS
	#if PERMUTATION_CULLING	
	const uint NumControlPoints = HairStrandsVF_bCullingEnable ? HairStrandsVF_CullingIndirectBuffer[3] : ControlPointCount;
	#else // PERMUTATION_CULLING
	const uint NumControlPoints = ControlPointCount;
	#endif //PERMUTATION_CULLING
#endif // PERMUTATION_INDIRECT_PRIM_IDS

	const uint NumBatches = (NumControlPoints + 63) / 64;
	const uint NumLoops = (NumBatches + (NumWorkGroups - 1)) / NumWorkGroups;
	const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

	LOOP for (uint LoopIndex = 0; LoopIndex < NumLoops; ++LoopIndex)
	{
		const uint BatchIndex = LoopIndex + (GroupID * NumLoops);
		bool bSegValid = (BatchIndex < NumBatches);

#if PERMUTATION_INDIRECT_PRIM_IDS
		uint PrimID = 0;
		const uint PrimIDIndex = BatchIndex * 64 + GroupThreadID;
		bSegValid = bSegValid && (PrimIDIndex < NumControlPoints);
		if (bSegValid)
		{
			PrimID = IndirectPrimIDs[PrimIDIndex];
		}
#else // PERMUTATION_INDIRECT_PRIM_IDS
	#if PERMUTATION_CULLING
		uint PrimID = BatchIndex * 64 + GroupThreadID;
		bSegValid = bSegValid && (PrimID < NumControlPoints);

		if (bSegValid && HairStrandsVF_bCullingEnable)
		{
			const uint FetchIndex0 = PrimID;
			const uint FetchIndex1 = min(FetchIndex0 + 1, NumControlPoints - 1);

			const uint VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
			const uint VertexIndex1 = HairStrandsVF_CullingIndexBuffer[FetchIndex1];

			if (VertexIndex1 != VertexIndex0 + 1)
			{
				bSegValid = false;
			}
			else
			{
				PrimID = VertexIndex0;
			}
		}
	#else // PERMUTATION_CULLING
		const uint PrimID = BatchIndex * 64 + GroupThreadID;
		bSegValid = bSegValid && (PrimID < ControlPointCount);
	#endif // PERMUTATION_CULLING
#endif // PERMUTATION_INDIRECT_PRIM_IDS

		if (!bSegValid)
		{
			continue;
		}

		// Fetch both control points
		FHairControlPoint CP0 = ReadHairControlPoint(
			HairStrandsVF_PositionBuffer,
			PrimID,
			PositionOffset,
			HairStrandsVF_Radius,
			HairStrandsVF_RootScale,
			HairStrandsVF_TipScale);

		if (CP0.Type == HAIR_CONTROLPOINT_END)
		{
			continue;
		}

		const FHairControlPoint CP1 = ReadHairControlPoint(
			HairStrandsVF_PositionBuffer,
			PrimID + 1,
			PositionOffset,
			HairStrandsVF_Radius,
			HairStrandsVF_RootScale,
			HairStrandsVF_TipScale);

		// Transform to world space
		const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
		const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
		float4 SP0 = mul(float4(CP0WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
		float4 SP1 = mul(float4(CP1WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));

		if (!BlinnLineClipping(SP0, SP1))
		{
			continue;
		}

		SP0 = float4(NDCToPixelCoord(SP0), 1.0f / SP0.w);
		SP1 = float4(NDCToPixelCoord(SP1), 1.0f / SP1.w);

		const float SegmentLenSqRcp = 1.0f / dot(SP1.xy - SP0.xy, SP1.xy - SP0.xy);

		const float2 DeltaAbs = abs(SP1.xy - SP0.xy);
		const uint NumSteps = (uint)ceil(max(DeltaAbs.x, DeltaAbs.y)) + 1;
		const float RcpNumSteps = NumSteps > 1 ? (1.0f / (NumSteps - 1)) : 1.0f;
		const bool bIsSteep = DeltaAbs.y > DeltaAbs.x;
		const uint PrimMatID = PackHairVisControlPointMaterialId(PrimID, HairMaterialId);

		LOOP for (int J = 0; J < NumSteps; ++J)
		{
			const float Alpha = J * RcpNumSteps;
			const float4 SP = lerp(SP0, SP1, Alpha);

			const float AntiAliasingFactor = 1.0f;
			Plot(SP.xy, AntiAliasingFactor, SP0, SP1, CP0.WorldRadius * 2000.0f, CP1.WorldRadius * 2000.0f, SegmentLenSqRcp, PrimMatID);
		}
	}
}