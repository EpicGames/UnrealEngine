// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"

#if SHADER_RASTER_HW_VS

#ifndef PERMUTATION_DRAW_INDIRECT
#define PERMUTATION_DRAW_INDIRECT 0
#endif

#ifndef PERMUTATION_CULLING
#define PERMUTATION_CULLING 0
#endif

Buffer<uint> 		RasterPrimIDs;
float3				ViewDir;
uint				HairMaterialId;
float3				CameraOrigin;
float				RadiusAtDepth1;
uint				PrimIDsBufferSize;


struct VSInput
{
	uint VertexID : SV_VertexID;
	uint InstanceID : SV_InstanceID;
};

struct VSOutput
{
	float4 Position : SV_Position;
	float Radius : RADIUS;
	nointerpolation uint PrimMatID : PRIM_MAT_ID;
};

VSOutput GetNullOutput()
{
	VSOutput Output = (VSOutput)0;
	Output.Position = INFINITE_FLOAT;
	return Output;
}

VSOutput VSMain(VSInput Input)
{
#if PERMUTATION_DRAW_INDIRECT
	const uint PrimID = RasterPrimIDs[PrimIDsBufferSize - 1 - Input.InstanceID];
#else // PERMUTATION_DRAW_INDIRECT
	#if PERMUTATION_CULLING
	uint PrimID = Input.InstanceID;

	if (HairStrandsVF_bCullingEnable)
	{
		const uint NumVerticesAfterCulling = HairStrandsVF_CullingIndirectBuffer[3];
		if (PrimID >= NumVerticesAfterCulling)
		{
			return GetNullOutput();
		}
		const uint FetchIndex0 = PrimID;
		const uint FetchIndex1 = min(FetchIndex0 + 1, NumVerticesAfterCulling - 1);

		const uint VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
		const uint VertexIndex1 = HairStrandsVF_CullingIndexBuffer[FetchIndex1];

		if (VertexIndex1 != VertexIndex0 + 1)
		{
			return GetNullOutput();
		}
		else
		{
			PrimID = VertexIndex0;
		}
	}
	#else
	const uint PrimID = Input.InstanceID;
	#endif // PERMUTATION_CULLING
#endif // PERMUTATION_DRAW_INDIRECT

	const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

	// Fetch both control points
	const FHairControlPoint CP0 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		PrimID,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	// Kill quad
	if (CP0.Type == HAIR_CONTROLPOINT_END)
	{
		return GetNullOutput();
	}

	const FHairControlPoint CP1 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		PrimID + 1,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	// Transform to world space
	const float3 CP0WP = mul(float4(CP0.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	const float3 CP1WP = mul(float4(CP1.Position, 1.0f), HairStrandsVF_LocalToWorldPrimitiveTransform).xyz;
	
	const float3 CPWP = Input.VertexID < 2 ? CP0WP : CP1WP;
	const float CPWorldRadius = Input.VertexID < 2 ? CP0.WorldRadius : CP1.WorldRadius;

	// Minimal radius to snap the strand to a sample/pixel center (to avoid aliasing)
	const float DistanceToCamera = GetDistanceToCameraFromViewVector(CameraOrigin - CPWP);
	const float MinStrandHairRadius = ConvertGivenDepthRadiusForProjectionType(RadiusAtDepth1, DistanceToCamera);

	const float3 Tangent = normalize(CP1WP - CP0WP);
	const float3 Right = normalize(cross(Tangent, ViewDir));
	
	const float3 WP = CPWP
		+ (Input.VertexID == 1 || Input.VertexID == 3 ? Right : -Right)
		* max(CPWorldRadius, MinStrandHairRadius);

	VSOutput Output = (VSOutput)0;
	
	Output.Position = mul(float4(WP, 1.0f), DFHackToFloat(PrimaryView.WorldToClip));
	Output.Radius = CPWorldRadius * 2000.0f;
	Output.PrimMatID = PackHairVisControlPointMaterialId(PrimID, HairMaterialId);

	return Output;
}

#endif // SHADER_RASTER_HW_VS

#if SHADER_RASTER_HW_PS

RWTexture2D<uint>	OutHairCountTexture;
RWTexture2D<uint>	OutDepthCovTexture;
RWTexture2D<uint>	OutPrimMatTexture;
float				CoverageScale;

EARLYDEPTHSTENCIL
void PSMain(
	float4 Position : SV_Position,
	float Radius : RADIUS,
	nointerpolation uint PrimMatID : PRIM_MAT_ID
)
{
	const uint2 Coord = Position.xy;
	const uint PackedDepthCov = PackHairVisDepthCoverage(Position.z, 1.0f);

	InterlockedAdd(OutHairCountTexture[Coord], min(Radius / Position.w, 0.5f) * 2.0f * 1000.0f * CoverageScale);

	// Write Depth + PrimMatID if depth test against hair depths is passed
	uint OldValue;
	InterlockedMax(OutDepthCovTexture[Coord], PackedDepthCov, OldValue);
	if (PackedDepthCov > OldValue)
	{
		OutPrimMatTexture[Coord] = PrimMatID;
	}
}

#endif // SHADER_RASTER_HW_PS