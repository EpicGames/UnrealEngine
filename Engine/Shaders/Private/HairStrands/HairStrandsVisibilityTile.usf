// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "../Common.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsTileCommon.ush"

#define GROUP_THREAD_COUNT (HAIR_TILE_SIZE*HAIR_TILE_SIZE)

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_GENERATION
int2 BufferResolution;
uint bUintTexture;
uint bForceOutputAllTiles;
float TransmittanceThreshold;
uint IntCoverageThreshold;
#if PERMUTATION_INPUT_TYPE == 0
Texture2D<float> InputFloatTexture;
#elif PERMUTATION_INPUT_TYPE == 1
Texture2D<uint> InputUintTexture;
#else
	#error Input type not defined
#endif
RWBuffer<uint> TileCountBuffer;

RWBuffer<uint2> TileHairAllBuffer;
RWBuffer<uint2> TileHairFullBuffer;
RWBuffer<uint2> TileHairPartialBuffer;
RWBuffer<uint2> TileOtherBuffer;

#if PERMUTATION_WAVEOPS == 0
// 8x8 bit mask
groupshared uint2 s_fMask[GROUP_THREAD_COUNT];
groupshared uint2 s_pMask[GROUP_THREAD_COUNT];
#endif

#if PERMUTATION_WAVE_OPS && COMPILER_SUPPORTS_WAVE_SIZE
WAVESIZE(64) // PERMUTATION_WAVE_OPS is true only when wave>=64 are available
#endif
[numthreads(HAIR_TILE_SIZE, HAIR_TILE_SIZE, 1)]
void TileMainCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint LinearIndex : SV_GroupIndex, uint3 GroupId : SV_GroupID)
{
	const uint2 PixelCoord = DispatchThreadId + View.ViewRectMin.xy;
	const bool bIsValid    = all(DispatchThreadId < uint2(View.ViewRectMinAndSize.zw));

	bool fHair = false;
	bool pHair = false;
	if (bIsValid)
	{
		if (bForceOutputAllTiles)
		{
			// When bForceOutputAllTiles, we generate tiles for all pixels.
			fHair = true;
			pHair = false;
		}
		else
		#if PERMUTATION_INPUT_TYPE == 0
		{
			const float Transmittance = InputFloatTexture.Load(uint3(PixelCoord, 0));
			fHair = Transmittance <= TransmittanceThreshold;
			pHair = Transmittance < 1.f;
		}
		#elif PERMUTATION_INPUT_TYPE == 1
		{
			const uint Coverage = InputUintTexture.Load(uint3(PixelCoord, 0));
			fHair = Coverage >= IntCoverageThreshold;
			pHair = Coverage > 0;//&& Coverage < IntCoverageThreshold; //this is now consistent with non-int branch below
		}
		#endif
	}
#if PERMUTATION_WAVEOPS
	const bool bIsFull    = WaveActiveAllTrue(fHair);
	const bool bIsPartial = WaveActiveAnyTrue(pHair);
	if (all(LinearIndex == 0)) 
	{
		if (bIsFull || bIsPartial)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_ALL], 1, WriteToIndex);
			TileHairAllBuffer[WriteToIndex] = GroupId.xy;
		}
		if (bIsFull)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_FULL], 1, WriteToIndex);
			TileHairFullBuffer[WriteToIndex] = GroupId.xy;
		}
		else if (bIsPartial)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_PARTIAL], 1, WriteToIndex);
			TileHairPartialBuffer[WriteToIndex] = GroupId.xy;
		}
		else
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_OTHER], 1, WriteToIndex);
			TileOtherBuffer[WriteToIndex] = GroupId.xy;
		}
	}
#else
	const uint2 Mask = LinearIndex < 32 ? uint2(1u << LinearIndex, 0u) : uint2(0u, 1u << (LinearIndex - 32u));
	s_fMask[LinearIndex] = fHair ? Mask : 0u;
	s_pMask[LinearIndex] = pHair ? Mask : 0u;

	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 32)
	{
		s_fMask[LinearIndex] = s_fMask[LinearIndex] | s_fMask[LinearIndex + 32];
		s_pMask[LinearIndex] = s_pMask[LinearIndex] | s_pMask[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		s_fMask[LinearIndex] = s_fMask[LinearIndex] | s_fMask[LinearIndex + 16];
		s_pMask[LinearIndex] = s_pMask[LinearIndex] | s_pMask[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 8)
	{
		s_fMask[LinearIndex] = s_fMask[LinearIndex] | s_fMask[LinearIndex + 8];
		s_pMask[LinearIndex] = s_pMask[LinearIndex] | s_pMask[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		s_fMask[LinearIndex] = s_fMask[LinearIndex] | s_fMask[LinearIndex + 4];
		s_pMask[LinearIndex] = s_pMask[LinearIndex] | s_pMask[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		s_fMask[LinearIndex] = s_fMask[LinearIndex] | s_fMask[LinearIndex + 2];
		s_pMask[LinearIndex] = s_pMask[LinearIndex] | s_pMask[LinearIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 1)
	{
		const uint2 fMask = s_fMask[LinearIndex] | s_fMask[LinearIndex + 1];
		const uint2 pMask = s_pMask[LinearIndex] | s_pMask[LinearIndex + 1];

		const bool bIsFull = all(fMask == 0xFFFFFFFF);
		const bool bIsPartial = any(pMask > 0);
			
		if (bIsFull || bIsPartial)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_ALL], 1, WriteToIndex);
			TileHairAllBuffer[WriteToIndex] = GroupId.xy;
		}

		if (bIsFull)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_FULL], 1, WriteToIndex);
			TileHairFullBuffer[WriteToIndex] = GroupId.xy;
		}
		else if (bIsPartial)
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_HAIR_PARTIAL], 1, WriteToIndex);
			TileHairPartialBuffer[WriteToIndex] = GroupId.xy;
		}
		else
		{
			uint WriteToIndex;
			InterlockedAdd(TileCountBuffer[HAIRTILE_OTHER], 1, WriteToIndex);
			TileOtherBuffer[WriteToIndex] = GroupId.xy;
		}
	}
#endif
}
#endif // SHADER_TILE_GENERATION

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_COPY_ARGS
uint2 TileCountXY;
uint  TilePerThread_GroupSize;
uint  bRectPrimitive;
uint  TileSize;
Buffer<uint> TileCountBuffer;
RWBuffer<uint> TileIndirectDrawBuffer;
RWBuffer<uint> TileIndirectDispatchBuffer;
RWBuffer<uint> TilePerThreadIndirectDispatchBuffer;
RWBuffer<uint> TileIndirectRayDispatchBuffer;

void FillIndirectBuffers(uint TileType)
{
	const uint TileCount = TileCountBuffer[TileType];
	
	// Indirect draw
	TileIndirectDrawBuffer[TileType * 4 + 0] = bRectPrimitive > 0 ? 4 : 6;  // VertexCountPerInstance
	TileIndirectDrawBuffer[TileType * 4 + 1] = TileCount;  // InstanceCount
	TileIndirectDrawBuffer[TileType * 4 + 2] = 0;  // StartVertexLocation
	TileIndirectDrawBuffer[TileType * 4 + 3] = 0;  // StartInstanceLocation

	// Indirect dispatch
	WriteDispatchIndirectArgs(TileIndirectDispatchBuffer, TileType,
		TileCount >= TileCountXY.x ? TileCountXY.x : TileCount,
		DivideAndRoundUp(TileCount, TileCountXY.x),
		1);

	// Indirect dispatch with one thread
	// At 4k, with 8x8 tiles: 4096 x 4096 / 8 x 8 = 512 x 512 tiles 
	// With a group size of 64 (8x8), the total dispatch count should be 512 x 512 / 8 x 8 = 64 x 64 = 4096, which is below the 65k limit per dimension
	WriteDispatchIndirectArgs(TilePerThreadIndirectDispatchBuffer, TileType,
		DivideAndRoundUp(TileCount, TilePerThread_GroupSize),
		1,
		1);

	// Indrect ray dispatch
	// Ray tracing dispatch dimensions are defined simply in terms of threads/rays, not thread groups.
	WriteDispatchIndirectArgs(TileIndirectRayDispatchBuffer, TileType,
		TileCount * TileSize,
		TileSize,
		1);
}

[numthreads(HAIRTILE_COUNT, 1, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.y == 0 && DispatchThreadId.x < HAIRTILE_COUNT)
	{
		uint HairTileType = DispatchThreadId.x; // HairAll, HairFull, HairPartial, Other
		FillIndirectBuffers(HairTileType);
	}
}
#endif // SHADER_TILE_COPY_ARGS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_DEBUG

float3 DebugColor;

void MainPS(
	in FScreenVertexOutput Input,
	out float4 OutColor0 : SV_Target0)
{
	const uint2 InTileCoord = uint2(Input.Position.xy) >> 3u; // HAIR_TILE_SIZE == 8;
	const bool bTileX = (InTileCoord.x & 1) == 0;
	const bool bTileY = (InTileCoord.y & 1) == 0;
	const bool bChecker = (bTileX && bTileY) || (!bTileX && !bTileY);
	OutColor0 = 255;
	OutColor0 = bChecker ? float4(DebugColor * 0.5f, 1.0f) : float4(DebugColor * 1.0f, 1.0f);
}

#endif //SHADER_TILE_DEBUG

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_DEBUG_PRINT
#include "../ShaderPrint.ush"

int2 MaxResolution;
uint TileGroupSize;
uint TileSize;
uint TileCount;
uint TileType;
int2 TileCountXY;
uint bRectPrimitive;

void AddTileCountLine(inout FShaderPrintContext Context, uint TileType)
{
	const uint ActiveTileCount = HairStrands.HairTileCount[TileType];
	Print(Context, TEXT("Count "), FontWhite);
	if (TileType == HAIRTILE_HAIR_ALL)
	{
		Print(Context, TEXT("Hair(All)    "), FontEmerald);
	}
	else if (TileType == HAIRTILE_HAIR_FULL)
	{
		Print(Context, TEXT("Hair(Full)   "), FontEmerald);
	}
	else if (TileType == HAIRTILE_HAIR_PARTIAL)
	{
		Print(Context, TEXT("Hair(Partial)"), FontEmerald);
	}
	else if (TileType == HAIRTILE_OTHER)
	{
		Print(Context, TEXT("Clear        "), FontEmerald);
	}
	else
	{
		Print(Context, TEXT("Unknown      "), FontEmerald);
	}
	Print(Context, TEXT(": "), FontWhite);
	
	Print(Context, ActiveTileCount, FontYellow);
	Print(Context, 100.f * float(ActiveTileCount) / float(TileCount), FontOrange);
	Print(Context, TEXT("%"), FontOrange);
	Newline(Context);
}

[numthreads(1, 1, 1)]
void MainCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId != 0)) return;

	// Pixel coord
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

	Print(Context, TEXT("Tile Count XY      : "), FontWhite);
	Print(Context, TileCountXY, FontEmerald);
	Newline(Context);

	Print(Context, TEXT("Tile type          : "), FontWhite);
	FFontColor HairAllColor = FontOrange;
	FFontColor HairFullColor = FontOrange;
	FFontColor HairPartialColor = FontOrange;
	FFontColor OtherColor = FontOrange;
	if (TileType == HAIRTILE_HAIR_ALL)
	{
		Print(Context, TEXT("Hair(All)"), FontOrange);
		HairAllColor = FontOrange;
	}
	else if (TileType == HAIRTILE_HAIR_FULL)
	{
		Print(Context, TEXT("Hair(Full)"), FontOrange);
		HairFullColor = FontOrange;
	}
	else if (TileType == HAIRTILE_HAIR_PARTIAL)
	{
		Print(Context, TEXT("Hair(Partial)"), FontOrange);
		HairPartialColor = FontOrange;
	}
	else if (TileType == HAIRTILE_OTHER)
	{
		Print(Context, TEXT("Clear"), FontOrange);
		OtherColor = FontOrange;
	}
	else
	{
		Print(Context, TEXT("Unknown"), FontOrange);
	}
	Newline(Context);
	Print(Context, TEXT("Tile Size          : "), FontWhite);
	Print(Context, TileSize, FontYellow);
	Newline(Context);

	Print(Context, TEXT("Tile Threads       : "), FontWhite);
	Print(Context, TileGroupSize, FontYellow);
	Newline(Context);

	Print(Context, TEXT("Primitive          : "), FontWhite);
	if (bRectPrimitive)
		Print(Context, TEXT("Rect."), FontOrange);
	else
		Print(Context, TEXT("Triangle"), FontOrange);
	Newline(Context);

	Newline(Context);
	AddTileCountLine(Context, HAIRTILE_HAIR_ALL);
	AddTileCountLine(Context, HAIRTILE_HAIR_FULL);
	AddTileCountLine(Context, HAIRTILE_HAIR_PARTIAL);
	AddTileCountLine(Context, HAIRTILE_OTHER);

}

#endif //SHADER_TILE_DEBUG_PRINT

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_VS
int2 ViewMin;
float2 ViewInvSize;
uint TileType;
Buffer<uint2> TileDataBuffer;

void MainVS(
	in uint InVertexId : SV_VertexID,
	in uint InInstanceId : SV_InstanceID,
	out FScreenVertexOutput Out)
{
	Out = (FScreenVertexOutput)0;
	const uint2 TileCoord = TileDataBuffer[InInstanceId];

	uint2 TileVertex = TileCoord * HAIR_TILE_SIZE;
	TileVertex.x += InVertexId == 1 || InVertexId == 2 || InVertexId == 4 ? HAIR_TILE_SIZE : 0;
	TileVertex.y += InVertexId == 2 || InVertexId == 4 || InVertexId == 5 ? HAIR_TILE_SIZE : 0;
	Out.UV = float2(TileVertex) * ViewInvSize; // No need to take ViewMin into account, as the viewport rect is center on the actual view (i.e., xy:ViewMin)
	Out.Position = float4(Out.UV * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);
}

#endif //SHADER_TILE_VS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_CLEAR

uint TileSize;
uint TileType;
int2 TileCountXY;
int2 ViewRectMin;
int2 Resolution;
Buffer<uint>  TileCountBuffer;
Buffer<uint2> TileDataBuffer;

RWTexture2D<float4> OutTexture;

[numthreads(HAIR_TILE_SIZE, HAIR_TILE_SIZE, 1)]
void TileMainCS(uint2 InGroupId : SV_GroupID, uint2 InGroupThreadId : SV_GroupThreadID)
{
	const uint TileCount   = TileCountBuffer[TileType];
	const uint TileIndex1D = InGroupId.x + InGroupId.y * TileCountXY.x;
	if (TileIndex1D >= TileCount)
	{
		return;
	}
	const uint2 GroupId = TileDataBuffer[TileIndex1D];
	const uint2 PixelCoord = uint2(ViewRectMin) + GroupId * TileSize + InGroupThreadId;

	if (all(PixelCoord < uint2(Resolution)))
	{
		OutTexture[PixelCoord] = 0;
	}
}
#endif // SHADER_TILE_CLEAR