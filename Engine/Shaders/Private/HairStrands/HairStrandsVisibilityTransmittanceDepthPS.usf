// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "HairStrandsVisibilityCommon.ush"

Texture2D<float> SceneDepthTexture;
Texture2D<float> HairOnlyDepthTexture;

float DistanceThreshold;
void MainPS(
	in FScreenVertexOutput Input, 
#if PERMUTATION_OUTPUT_FORMAT == 0
	out float OutColor : SV_Target0
#elif PERMUTATION_OUTPUT_FORMAT == 1
	out float2 OutColor : SV_Target0
#endif
)
{
#if 1
	OutColor = 0;
#else
	const uint3 PixelCoord = uint3(Input.Position.xy, 0);
	const float FarDepth = 10e-06;

	const float SceneDepth  = ConvertFromDeviceZ(SceneDepthTexture.Load(PixelCoord));
	const float SampleDepth = ConvertFromDeviceZ(HairOnlyDepthTexture.Load(PixelCoord));

	if (abs(SampleDepth-SceneDepth) > DistanceThreshold)
	{
		discard;
	}

	const uint LargeHairCount = 100000;
#if PERMUTATION_OUTPUT_FORMAT == 0
	OutColor = LargeHairCount;
#elif PERMUTATION_OUTPUT_FORMAT == 1
	OutColor = float2(LargeHairCount, 1);
#endif

#endif
}
