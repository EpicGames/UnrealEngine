// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Convert a FHairSample into a FGufferData
FGBufferData HairSampleToGBufferData(in FHairSample In, in float InDualScatteringRoughnessOverride=0)
{
	FGBufferData Out = (FGBufferData)0;
	Out.ShadingModelID = SHADINGMODELID_HAIR;
	Out.WorldNormal = In.Tangent;
	Out.BaseColor = In.BaseColor;
	Out.Roughness = In.Roughness;
	Out.Specular = In.Specular;
	Out.Metallic = 0; // Scattering;
	Out.Depth = ConvertFromDeviceZ(In.Depth);
	Out.GBufferAO = 1;
	Out.CustomData = float4(InDualScatteringRoughnessOverride, 0, In.Backlit, 0);
	Out.IndirectIrradiance = 1000000;
	Out.DiffuseColor = In.BaseColor;
	Out.SpecularColor = DielectricSpecularToF0(In.Specular);
	Out.PrecomputedShadowFactors = 1;
	Out.PerObjectGBufferData = HasHairFlags(In.Flags, HAIR_FLAGS_CAST_CONTACT_SHADOW) ? 1u : 0u;
	return Out;
}