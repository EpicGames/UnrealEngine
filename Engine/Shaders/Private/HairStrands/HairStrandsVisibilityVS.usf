// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../VelocityCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush" 
#include "HairStrandsVisibilityCommon.ush"

struct FHairVisibilityAccumulateVSToPS
{
#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
	nointerpolation uint HairControlPointId : HAIR_PRIMITIVE_ID;
#else
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
#endif
	float4 Position : SV_POSITION;
#if HAIR_RENDER_MODE != RENDER_MODE_MSAA_VISIBILITY
	float WorldStrandRadius : TEXCOORD8;
#endif
#if HAIR_RENDER_MODE == RENDER_MODE_PPLL
	INVARIANT_OUTPUT float4 PackedVelocityA : TEXCOORD9;
	INVARIANT_OUTPUT float4 PackedVelocityC : TEXCOORD12;
#endif
};

#define VS_OUTPUT_TYPE FHairVisibilityAccumulateVSToPS

#if VERTEXSHADER 

void Main(
	FVertexFactoryInput Input,
	out VS_OUTPUT_TYPE Output
)
#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY && HAIR_STRAND_MESH_FACTORY
{
	ResolvedView = ResolveView();

	const bool bUseStableRasterization = UseStableRasterization();
	const FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits, bUseStableRasterization);
	const float VelocityScale = 0;

	FHairViewInfo HairViewInfo;
	HairViewInfo.TranslatedWorldCameraOrigin = ResolvedView.TranslatedWorldCameraOrigin;
	HairViewInfo.ViewForward = ResolvedView.ViewForward;
	HairViewInfo.RadiusAtDepth1 = lerp(HairRenderInfo.RadiusAtDepth1Primary, HairRenderInfo.RadiusAtDepth1Velocity, VelocityScale);
	HairViewInfo.bIsOrthoView = HairRenderInfo.bIsOrthoView;

	uint HairControlPointId = 0;
	const float4 WorldPosition = VertexFactoryGetWorldPosition_Visibility(Input, HairViewInfo, HairControlPointId);
	Output.Position = mul(WorldPosition, View.TranslatedWorldToClip);
	Output.HairControlPointId = HairControlPointId;
}
#else // HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
{
	ResolvedView = ResolveView();

	// #hair_todo: This is a hack/quick term solution, in order to generate valide light/view direction in the case of hair strand factory, as this is used for alighning the strand with the view.
	//ResolvedView.TranslatedWorldCameraOrigin = -HairVisibilityPass.LightDirection * 1000; // Hack for strand rendering which use the camera position for deriving the view vector for forcing strand view facing
	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
#if HAIR_STRAND_MESH_FACTORY
	// Use a raw position (rather than quad facing position) as the pre-position is not expanded into quad (for perf. reason)
	float4 WorldPositionRaw = VertexFactoryGetWorldPositionRaw(Input, VFIntermediates);
#else
	float4 WorldPositionRaw = VertexFactoryGetWorldPosition(Input, VFIntermediates);
#endif
	
	bool bUseStableRasterization = false;
#if HAIR_STRAND_MESH_FACTORY
	bUseStableRasterization = UseStableRasterization();
#endif

	// Velocity computation
	// The velocity is used to adapt/increase the minimum rasterization size in order to avoid aliasing under heavy-motion.
	// The higher the velocity, the larger a strand becomes.
	float VelocityScale = 0;
	const FHairRenderInfo HairRenderInfo = GetHairRenderInfo(ResolvedView.HairRenderInfo, ResolvedView.HairRenderInfoBits, bUseStableRasterization);
	#if HAIR_RENDER_MODE == RENDER_MODE_PPLL
	{
		const float4 PrevWorldPosition	= VertexFactoryGetPreviousWorldPosition(Input, VFIntermediates);
		const float4 ScreenPos			= mul(float4(WorldPositionRaw.xyz, 1), ResolvedView.TranslatedWorldToClip);
		const float4 PrevScreenPosObj	= mul(float4(PrevWorldPosition.xyz, 1), ResolvedView.PrevTranslatedWorldToClip);
		Output.PackedVelocityA			= INVARIANT(ScreenPos);
		Output.PackedVelocityC			= INVARIANT(PrevScreenPosObj);

		const float2 ScreenVelocity = Calculate3DVelocity(Output.PackedVelocityA, Output.PackedVelocityC).xy;
		VelocityScale = saturate(max(abs(ScreenVelocity.x), abs(ScreenVelocity.y)) / HairRenderInfo.VelocityMagnitudeScale);
	}
	#endif
	
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

#if HAIR_STRAND_MESH_FACTORY
	FHairViewInfo HairViewInfo;
	HairViewInfo.TranslatedWorldCameraOrigin = ResolvedView.TranslatedWorldCameraOrigin;
	HairViewInfo.ViewForward = ResolvedView.ViewForward;
	HairViewInfo.RadiusAtDepth1 = lerp(HairRenderInfo.RadiusAtDepth1Primary, HairRenderInfo.RadiusAtDepth1Velocity, VelocityScale);
	HairViewInfo.bIsOrthoView = HairRenderInfo.bIsOrthoView;

	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates, HairViewInfo);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal, HairViewInfo);
#else
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
#endif

	// Isolate instructions used for world position offset on xbox 360, 
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	//ISOLATE
	//{
	//	WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	//	WorldPositionRaw.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	//	PrevWorldPosition.xyz += GetMaterialPreviousWorldPositionOffset(VertexParameters); 
	//}

	Output.Position = mul(WorldPosition, View.TranslatedWorldToClip);

#if HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY
	Output.HairControlPointId = VFIntermediates.HairControlPointId;
#else
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

  #ifdef HAIR_STRAND_MESH_FACTORY
	Output.WorldStrandRadius = VFIntermediates.HairDimensions.y;
  #else
	Output.WorldStrandRadius = 1;
  #endif
#endif
}
#endif // HAIR_RENDER_MODE == RENDER_MODE_MSAA_VISIBILITY

#endif // VERTEXSHADER