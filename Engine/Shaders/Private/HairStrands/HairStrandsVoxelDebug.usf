// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush" 
#include "../SceneTextureParameters.ush"
#include "../DeferredShadingCommon.ush"
#include "../ShaderPrint.ush"
#include "../PositionReconstructionCommon.ush"  
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsVoxelPageCommon.ush"
#include "../ColorMap.ush"  

#if SHADER_VOXEL_DEBUG

#define VOXEL_TRAVERSAL_DEBUG 0
#define VOXEL_TRAVERSAL_FORCE_MIP_ENABLE 1
#if PERMUTATION_TRAVERSAL == 1
	#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#else
	#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR
#endif
#include "HairStrandsVoxelPageTraversal.ush"

float2 OutputResolution; 
uint MacroGroupId;
uint MacroGroupCount;
uint MaxTotalPageIndexCount;

Texture2D<float> SceneDepthBeforeCompositionTexture;
RWTexture2D<float4> OutputTexture;
Buffer<uint> TotalValidPageCounter;

#define VOXEL_BLOCK 0 
#define VOXEL_SMOOTH 1
#define VOXEL_FILTERING VOXEL_BLOCK

uint GetVoxelPageIndex(uint3 InPageIndexCoord, uint3 InPageIndexResolution, uint InLinearPageIndexOffset)
{
	const uint LinearPageIndexCoord = CoordToIndex(InPageIndexCoord, InPageIndexResolution, InLinearPageIndexOffset);
	return VirtualVoxel.PageIndexBuffer.Load(LinearPageIndexCoord);
}

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const float2 PixelCoord = DispatchThreadId.xy + 0.5f;
	const float2 UV = PixelCoord / float2(OutputResolution);

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[MacroGroupId], VirtualVoxel.PageResolution);
	const bool bIsNodeValid = all(NodeDesc.PageIndexResolution != 0);

	const bool bPrint = all(uint2(PixelCoord) == 0);

	// Draw macro group AABB & debug infos
	if (bPrint)
	{
		const uint TotalPageIndexCount = MaxTotalPageIndexCount;
		const uint TotalPageCount = VirtualVoxel.PageCountResolution.x * VirtualVoxel.PageCountResolution.y * VirtualVoxel.PageCountResolution.z;
		const uint TotalAllocatedValidPageCount = TotalValidPageCounter.Load(0);
		uint TotalAllocatedPageCount = 0;
		for (uint MacroGroupIt = 0; MacroGroupIt < MacroGroupCount; ++MacroGroupIt)
		{
			const FPackedVirtualVoxelNodeDesc LocalPackedNode = VirtualVoxel.NodeDescBuffer[MacroGroupIt];
			const FVirtualVoxelNodeDesc LocalNodeDesc = UnpackVoxelNode(LocalPackedNode, VirtualVoxel.PageResolution);
			TotalAllocatedPageCount += LocalNodeDesc.PageIndexResolution.x * LocalNodeDesc.PageIndexResolution.y * LocalNodeDesc.PageIndexResolution.z;
		}

		AddAABBTWS(NodeDesc.TranslatedWorldMinAABB, NodeDesc.TranslatedWorldMaxAABB, ColorOrange);

		// General information
		// General
		//  Voxel World size        : 
		//  Page Resolution         :
		//  Page Count Resolution   :
		//  Page Texture Resolution :
		//
		//  Total Allocatled Page Index   :
		//  Total Page Index              :
		//  Total Page Index Occupancy    :
		//
		//  Total Allocated Page          :
		//  Total Page                    :
		//  Total Page Occupancy          :
		if (MacroGroupId == 0)
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 100));

			// Adaptive voxels
			Print(Context, TEXT("Alloc. Feedback  "), FontOrange);
			if (VirtualVoxel.AllocationFeedbackEnable)
			{
				Print(Context, TEXT("True"), FontGreen);
			}
			else
			{
				Print(Context, TEXT("False"), FontRed);
			}
			Newline(Context);

			// Actual voxel/pages size & resolution
			Print(Context, TEXT("GPU Curr. Voxel Size        "), FontOrange);
			Print(Context, VirtualVoxel.CurrGPUMinVoxelSize[0], FontEmerald);
			Newline(Context);

			Print(Context, TEXT("GPU Next. Voxel Size        "));
			Print(Context, VirtualVoxel.NextGPUMinVoxelSize[0]);
			Newline(Context);

			Print(Context, TEXT("CPU Min. Voxel Size         "));
			Print(Context, VirtualVoxel.CPUMinVoxelWorldSize);
			Newline(Context);
			
			Print(Context, TEXT("Page Res                    "));
			Print(Context, VirtualVoxel.PageResolution);
			Newline(Context);

			Print(Context, TEXT("Page Count Res              "));
			Print(Context, VirtualVoxel.PageCountResolution.x);
			Print(Context, VirtualVoxel.PageCountResolution.y);
			Print(Context, VirtualVoxel.PageCountResolution.z);
			Newline(Context);

			Print(Context, TEXT("Page Texture Res            "), FontEmerald);
			Print(Context, VirtualVoxel.PageTextureResolution.x, FontEmerald);
			Print(Context, VirtualVoxel.PageTextureResolution.y, FontEmerald);
			Print(Context, VirtualVoxel.PageTextureResolution.z, FontEmerald);
			Newline(Context);

			// Total page indices stats
			Newline(Context);

			Print(Context, TEXT("Total Allocated Page Index  "));
			Print(Context, TotalAllocatedPageCount);
			Newline(Context);

			Print(Context, TEXT("Total Page Index            "));
			Print(Context, TotalPageIndexCount);
			Newline(Context);

			const float PageIndexOccupancy = TotalAllocatedPageCount / float(TotalPageIndexCount);
			Print(Context, TEXT("Page Index Occupancy        "), FontOrange);
			Print(Context, PageIndexOccupancy * 100.f, GetOccupancyColor(PageIndexOccupancy));
			Newline(Context);

			// Total pages stats
			Newline(Context);

			Print(Context, TEXT("Total Allocated Valid Page  "));
			Print(Context, TotalAllocatedValidPageCount);
			Newline(Context);

			Print(Context, TEXT("Total Page                  "));
			Print(Context, TotalPageCount);
			Newline(Context);

			const float PageOccupancy = TotalAllocatedValidPageCount / float(TotalPageCount);
			Print(Context, TEXT("Page Occupancy              "), FontOrange);
			Print(Context, PageOccupancy * 100.f, GetOccupancyColor(PageOccupancy));
			Newline(Context);

			Print(Context, TEXT("Translated world offset     "), FontOrange);
			Print(Context, VirtualVoxel.TranslatedWorldOffset);
			Newline(Context);
		}

		// MacroGroupId XX
		//  Page Index Resolution: X Y Z (Linear/Total - %) 
		//  Virtual Resolution   : X Y Z
		//  Page Index Offset    : O
		if (MacroGroupId == 0)
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 420));

			Print(Context, TEXT("Macro Group        "), FontOrange);
			Newline(Context);

			Print(Context, TEXT(" Virtual Res X     "), FontEmerald);
			Newline(Context);
			Print(Context, TEXT(" Virtual Res Y     "), FontEmerald);
			Newline(Context);
			Print(Context, TEXT(" Virtual Res Z     "), FontEmerald);
			Newline(Context);

			Newline(Context);

			Print(Context, TEXT(" Page Index Res X  "));
			Newline(Context);
			Print(Context, TEXT(" Page Index Res Y  "));
			Newline(Context);
			Print(Context, TEXT(" Page Index Res Z  "));
			Newline(Context);

			Newline(Context);

			Print(Context, TEXT(" Page Index Offset "));
			Newline(Context);

			Newline(Context);

			Print(Context, TEXT(" Voxel World Size  "));
			Newline(Context);
		}

		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(250 + MacroGroupId * 35, 420));

			Print(Context, MacroGroupId, FontOrange);
			Newline(Context);

			Print(Context, NodeDesc.VirtualResolution.x, FontEmerald);
			Newline(Context);
			Print(Context, NodeDesc.VirtualResolution.y, FontEmerald);
			Newline(Context);
			Print(Context, NodeDesc.VirtualResolution.z, FontEmerald);
			Newline(Context);

			Newline(Context);

			Print(Context, NodeDesc.PageIndexResolution.x, FontYellow);
			Newline(Context);
			Print(Context, NodeDesc.PageIndexResolution.y, FontYellow);
			Newline(Context);
			Print(Context, NodeDesc.PageIndexResolution.z, FontYellow);
			Newline(Context);

			Newline(Context);

			// Offset of 1 line for every other voxel size, to avoid super-position with large float number
			const bool bAddLine = (MacroGroupId & 1) == 1;

			if (bAddLine) { Newline(Context); }

			Print(Context, NodeDesc.PageIndexOffset, FontSilver);
			Newline(Context);

			Newline(Context);

			Print(Context, NodeDesc.VoxelWorldSize, FontSilver);
			Newline(Context);
		}
	}


	// Draw widgets
	bool bDrawVoxel = true;
	bool bDrawPage = false;
	int ForcedMipLevel = -1;
	{
		FShaderPrintContext WidgetContext = InitShaderPrintContext(MacroGroupId == 0 && bPrint, uint2(500, 100));
		bDrawVoxel = AddCheckbox(WidgetContext, TEXT("Draw Voxels"), true, FontWhite); Newline(WidgetContext);
		bDrawPage = AddCheckbox(WidgetContext, TEXT("Draw Pages"), false, FontWhite); Newline(WidgetContext);
		bool  bForceMipLevel = AddCheckbox(WidgetContext, TEXT("Force MIP"), false, FontWhite); Newline(WidgetContext);
		if (bForceMipLevel)
		{
			ForcedMipLevel = 8u * AddSlider(WidgetContext, TEXT("MIP"), 0, FontWhite); 
			ForcedMipLevel = clamp(ForcedMipLevel, 0, int(VirtualVoxel.PageResolutionLog2));
			Print(WidgetContext, ForcedMipLevel, FontWhite, 1, 1);
			Print(WidgetContext, TEXT("/"), FontWhite);
			Print(WidgetContext, VirtualVoxel.PageResolutionLog2, FontWhite, 1, 1);
			Newline(WidgetContext);
		}
	}

	bool bIsValid = false;
	float3 HitColor = 0;
	if (bDrawPage)
	{
		// Reverse-Z
		const float SceneDepth = ConvertFromDeviceZ(SceneDepthBeforeCompositionTexture.Load(uint3(PixelCoord, 0)));
		const float3 O = PrimaryView.TranslatedWorldCameraOrigin;
		const float3 E = ReconstructTranslatedWorldPositionFromDepth(UV, SceneDepth);
		const float3 D = (E - O);

		float PixelRadius = -1;
		const float2 HitT = LineBoxIntersect(O, E, NodeDesc.TranslatedWorldMinAABB, NodeDesc.TranslatedWorldMaxAABB);
		float AccDensity = 0;
		if (HitT.x < HitT.y)
		{
			// Use different offseting to see either: 
			// * Individual voxel (for debugging mainly)
			// * Smoothed voxel (as used for transmittance computation)
		#if VOXEL_FILTERING == VOXEL_BLOCK
			const float3 Offset = 0;
			const float Dither = InterleavedGradientNoise(PixelCoord.xy, 1);
		#endif
		#if VOXEL_FILTERING == VOXEL_SMOOTH
			const float3 Offset = float3(InterleavedGradientNoise(PixelCoord.xy, 1), InterleavedGradientNoise(PixelCoord.xy, 2), InterleavedGradientNoise(PixelCoord.xy, 3));
			const float Dither = 0;
		#endif


			// Intersect the indirection table to see if a brick is empty or not
			const uint MaxStep = 256;
			const float Delta = (HitT.y-HitT.x)/float(MaxStep);
			for (uint StepIt=0; StepIt<MaxStep; ++StepIt)
			{			
				const float3 HitP = Offset + O + ((Dither+StepIt)*Delta + HitT.x) * D;
				const int3 PageIndexCoord = PositionToCoord(HitP, NodeDesc.TranslatedWorldMinAABB, NodeDesc.TranslatedWorldMaxAABB, NodeDesc.PageIndexResolution);
				const uint PageIndex = GetVoxelPageIndex(PageIndexCoord, NodeDesc.PageIndexResolution, NodeDesc.PageIndexOffset);

				if (PageIndex != INVALID_VOXEL_PAGE_INDEX)
				{
					const uint3 PageCoord = IndexToCoord(PageIndex, VirtualVoxel.PageCountResolution);

					AccDensity = 1;
					bIsValid = true;
					HitColor = PageIndexCoord / 12.f;
					break;
				}
			}
		}
	}
	else if (bDrawVoxel)
	{
		const bool bIsPixelEnable = all(int2(PixelCoord) == GetCursorPos());

		FHairTraversalResult TraversalResult = (FHairTraversalResult)0;

		const float DistanceThreshold = 10000;
		const bool bDebugEnabled = false;
		const float3 SampleRandom = GetHairVoxelJitter(PixelCoord, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);
		const float SceneDepth = ConvertFromDeviceZ(SceneDepthBeforeCompositionTexture.Load(uint3(PixelCoord, 0)));

		const float3 TranslatedWP0 = PrimaryView.TranslatedWorldCameraOrigin;
		const float3 TranslatedWP1 = ReconstructTranslatedWorldPositionFromDepth(UV, SceneDepth);

		FVirtualVoxelCommonDesc CommonDesc;
		CommonDesc.PageCountResolution	= VirtualVoxel.PageCountResolution;
		CommonDesc.PageTextureResolution= VirtualVoxel.PageTextureResolution;
		CommonDesc.PageResolution		= VirtualVoxel.PageResolution;
		CommonDesc.PageResolutionLog2	= VirtualVoxel.PageResolutionLog2;

		if (bIsNodeValid)
		{
			FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
			TraversalSettings.DensityScale		= VirtualVoxel.DensityScale;
			TraversalSettings.CountThreshold	= GetOpaqueVoxelValue();
			TraversalSettings.DistanceThreshold = DistanceThreshold;
			TraversalSettings.bDebugEnabled		= bDebugEnabled;
			TraversalSettings.SteppingScale		= VirtualVoxel.SteppingScale_Shadow;
			TraversalSettings.Random			= SampleRandom;
			TraversalSettings.TanConeAngle		= 0;
			TraversalSettings.bIsPrimaryRay		= true;
			TraversalSettings.PixelRadius		= -1;
			TraversalSettings.ForcedMip			= ForcedMipLevel;
			TraversalSettings.bDebugEnabled		= false; //bIsPixelEnable;

			TraversalResult = ComputeHairCountVirtualVoxel(
				TranslatedWP0,
				TranslatedWP1,
				CommonDesc,
				NodeDesc,
				VirtualVoxel.PageIndexBuffer,
				VirtualVoxel.PageTexture,
				TraversalSettings);

			HitColor = GetHSVDebugColor(saturate(TraversalResult.HairCount / 16));
			bIsValid = TraversalResult.HairCount > 0;
		}

		if (bIsPixelEnable && bIsValid)
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

			Print(Context, TEXT("X, Y "));
			Print(Context, GetCursorPos());
			Newline(Context);

			Print(Context, TEXT("T.Count "));
			Print(Context, TraversalResult.HairCount);
			Newline(Context);
			
			Print(Context, TEXT("T.Visibility "));
			Print(Context, TraversalResult.Visibility);
			Newline(Context);
		}
	}

	if (bIsValid)
	{
		OutputTexture[DispatchThreadId.xy] = float4(HitColor, 1);
	}
}

#endif // SHADER_VOXEL_DEBUG