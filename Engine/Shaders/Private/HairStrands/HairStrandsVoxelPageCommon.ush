// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairStrandsVoxelPageCommonStruct.ush"
#include "/Engine/Shared/HairStrandsDefinitions.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Define voxel traversal type
#define VOXEL_TRAVERSAL_NONE 0
#define VOXEL_TRAVERSAL_LINEAR 1
#define VOXEL_TRAVERSAL_SPARSE_LINEAR 2
#define VOXEL_TRAVERSAL_SPARSE_MIPMAP 3
#define VOXEL_TRAVERSAL_LINEAR_MIPMAP 4

///////////////////////////////////////////////////////////////////////////////////////////////////
uint3 PositionToCoord(float3 P, float3 InMinAABB, float3 InMaxAABB, uint3 InResolution)
{
	return clamp(
		uint3(saturate((P - InMinAABB) / (InMaxAABB - InMinAABB)) * InResolution),
		uint3(0,0,0),
		InResolution -1);
}

uint3 PositionToCoordUnclampled(float3 P, float3 InMinAABB, float3 InMaxAABB, uint3 InResolution)
{
	return uint3(saturate((P - InMinAABB) / (InMaxAABB - InMinAABB)) * float3(InResolution));
}

uint CoordToIndex(uint3 InCoord, uint3 InResolution, uint LinearOffset)
{
	return 
		InCoord.x +
		InCoord.y * InResolution.x +
		InCoord.z * InResolution.x * InResolution.y + 
		LinearOffset;
}

uint3 IndexToCoord(uint InIndex, uint3 InResolution)
{
	const uint SliceSize = (InResolution.x * InResolution.y);
	const uint SliceIndex = InIndex % SliceSize;

	uint3 OutCoord = 0;
	OutCoord.x = SliceIndex % InResolution.x;
	OutCoord.y = SliceIndex / InResolution.x;
	OutCoord.z = InIndex / SliceSize;

	return OutCoord;
}

bool IsInVoxelBounds(float3 P, float3 MinP, float3 MaxP)
{
	return
		P.x >= MinP.x && P.y >= MinP.y && P.z >= MinP.z &&
		P.x <= MaxP.x && P.y <= MaxP.y && P.z <= MaxP.z;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#define INVALID_VOXEL_PAGE_INDEX 0xFFFFFFFF
#define INVALID_MACRO_GROUP_ID 0xFF

#define VOXEL_CAST_NO_SHADOW_MASK	0x80000000	// Bit mask for holding if the voxel does *not* cast hair shadow (i.e. cast only transmission). The invert/negative logic is for making hair voxelization more efficient.
#define VOXEL_HAIR_MASK				0x00FFFFFF	// Bit mask for holding hair count/density
#define VOXEL_OPAQUE_MASK			0x7F000000	// Bit mask for holding opaque geometry
#define VOXEL_OPAQUE_ADD			0x7F000000
#define VOXEL_OPAQUE_SHIFT			24

struct FVirtualVoxelCommonDesc
{
	uint3  PageCountResolution;
	uint3  PageTextureResolution;
	uint   PageResolution;
	uint   PageResolutionLog2;
};

struct FVirtualVoxelNodeDesc
{
	float3	TranslatedWorldMinAABB; // In translated world space
	float3	TranslatedWorldMaxAABB; // In translated world space
	uint3	PageIndexResolution;
	uint3   VirtualResolution;
	uint    PageIndexOffset;
	float   VoxelWorldSize;
	bool    bIsValid;
};

uint PackUnorm11(float Value)
{
	return uint(saturate(Value) * 2047.0f);
}

float UnpackUnorm11(uint In)
{
	return float(In & 0x7FF) * (1.0f / 2047.0f);
}

uint PackVoxelWorldSize(float In)
{
	// Encode voxel world size [0..HAIR_MAX_VOXEL_SIZE] onto 10bits
	const float NormalizedVoxelWorldSize = min(In, HAIR_MAX_VOXEL_SIZE) * (1.f/HAIR_MAX_VOXEL_SIZE);
	return PackUnorm11(NormalizedVoxelWorldSize);
}
float UnpackVoxelWorldSize(uint In)
{
	return UnpackUnorm11(In) * HAIR_MAX_VOXEL_SIZE;
}

float QuantizeVoxelWorldSize(float In)
{
	return UnpackVoxelWorldSize(PackVoxelWorldSize(In));
}

FVirtualVoxelNodeDesc UnpackVoxelNode(FPackedVirtualVoxelNodeDesc In, uint InPageResolution)
{
	FVirtualVoxelNodeDesc Out;
	Out.TranslatedWorldMaxAABB = In.TranslatedWorldMaxAABB;
	Out.TranslatedWorldMinAABB = In.TranslatedWorldMinAABB;
	Out.PageIndexResolution = uint3(In.PackedPageIndexResolution & 0xFF, (In.PackedPageIndexResolution >> 8) & 0xFF, (In.PackedPageIndexResolution >> 16) & 0xFF);
	Out.VirtualResolution = Out.PageIndexResolution * InPageResolution;
	Out.PageIndexOffset = (In.PageIndexOffset_VoxelWorldSize & 0x1FFFFFu);
	Out.VoxelWorldSize = UnpackVoxelWorldSize(In.PageIndexOffset_VoxelWorldSize >> 21u);
	Out.bIsValid = all(Out.PageIndexResolution != 0);
	return Out;
}

FPackedVirtualVoxelNodeDesc PackVoxelNode(FVirtualVoxelNodeDesc In)
{
	FPackedVirtualVoxelNodeDesc Out;
	Out.TranslatedWorldMinAABB = In.TranslatedWorldMinAABB;
	Out.TranslatedWorldMaxAABB = In.TranslatedWorldMaxAABB;
	Out.PackedPageIndexResolution =
		 (In.PageIndexResolution.x & 0xFF) | 
		((In.PageIndexResolution.y & 0xFF)<<8) | 
		((In.PageIndexResolution.z & 0xFF)<<16);
	Out.PageIndexOffset_VoxelWorldSize =
		(In.PageIndexOffset & 0x1FFFFFu) |
		(PackVoxelWorldSize(In.VoxelWorldSize) << 21u);
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FHairTraversalResult
{
	float HairCount;
	float Visibility;
	float HitT;
};

FHairTraversalResult InitHairTraversalResult()
{
	FHairTraversalResult Out;
	Out.HairCount = 0;
	Out.Visibility = 1;
	Out.HitT = -1;
	return Out;
}

void Acc(inout FHairTraversalResult Out, const FHairTraversalResult In)
{
	Out.HairCount += In.HairCount;
	Out.Visibility = min(Out.Visibility, In.Visibility);
	Out.HitT = In.HitT >= 0 && Out.HitT >= 0 ? min(In.HitT, Out.HitT) : (In.HitT >= 0 ? In.HitT : Out.HitT);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

float GetVoxelDensityFixPointScale()
{
	// Constant for scaling the hair coverage during the voxelization (done with atomic integer)
	// TODO share or make common with the version in VisibilityCommon
	return 1000.f;
}

float GetOpaqueVoxelValue()
{
	// Constant for scaling the hair coverage during the voxelization (done with atomic integer)
	// TODO share or make common with the version in VisibilityCommon
	return VOXEL_OPAQUE_MASK / GetVoxelDensityFixPointScale() - 1;
}

// Return a value in [0..1] 0:no opaque/occlusion 1:fully occluded
float GetInternalVoxelOpaqueVisibility(uint RawDensity)
{
	const uint Raw = (RawDensity & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT;
	return 1.f - saturate(Raw / 255.f);
}

float GetInternalVoxelHairCount(uint RawDensity)
{
	const float VoxelFixPointScale = GetVoxelDensityFixPointScale();
	const uint Raw = (RawDensity & VOXEL_HAIR_MASK);
	return Raw / VoxelFixPointScale;
}

FHairTraversalResult GetHairVirtualVoxelDensity(uint3 InVoxelPageCoord, Texture3D<uint> InPageTexture, uint MipIndex, float DensityScale, bool bCastShadow)
{
	FHairTraversalResult Out = InitHairTraversalResult();
	const uint RawDensity = InPageTexture.Load(uint4(InVoxelPageCoord >> MipIndex, MipIndex));

	const bool bValid = !bCastShadow || (bCastShadow && (RawDensity & VOXEL_CAST_NO_SHADOW_MASK) == 0);
	if (bValid)
	{
		Out.HairCount  = GetInternalVoxelHairCount(RawDensity) * DensityScale;
		Out.Visibility = GetInternalVoxelOpaqueVisibility(RawDensity);
	}
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

uint EncodeBaseColorAndRoughness(float3 BaseColor, float Roughness)
{
	return
		(uint(BaseColor.x * 0xFF) & 0xFF) |
		(uint(BaseColor.y * 0xFF) & 0xFF) << 8 |
		(uint(BaseColor.z * 0xFF) & 0xFF) << 16 |
		(uint(Roughness * 0xFF) & 0xFF) << 24;
}

void DecodeBaseColorAndRoughness(uint Encoded, inout float3 BaseColor, out float Roughness)
{
	BaseColor.x = (float((Encoded) & 0xFF) / 255.f);
	BaseColor.y = (float((Encoded >> 8) & 0xFF) / 255.f);
	BaseColor.z = (float((Encoded >> 16) & 0xFF) / 255.f);
	Roughness = (float((Encoded >> 24) & 0xFF) / 255.f);
}
