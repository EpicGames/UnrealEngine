// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// VOXEL_TRAVERSAL_TYPE needs to be defined for selecting which type of traversal needs to be used
// Definition of traversal types are defined in HairStrandsVoxelPageCommon.ush
#ifndef VOXEL_TRAVERSAL_TYPE
#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_NONE
#endif

#ifndef VOXEL_TRAVERSAL_DEBUG
#define VOXEL_TRAVERSAL_DEBUG 0
#else
#include "../ShaderPrint.ush"
#endif

#ifndef VOXEL_TRAVERSAL_FORCE_MIP_ENABLE
#define VOXEL_TRAVERSAL_FORCE_MIP_ENABLE 0
#endif

#include "HairStrandsVoxelPageCommon.ush"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if VOXEL_TRAVERSAL_DEBUG 
void AddLine(bool bEnabled, float3 P0, float3 P1)
{
	if (!bEnabled)
		return;
	const float4 DebugColor = float4(1, 1, 0, 1);
	AddLineTWS(P0, P1, DebugColor, DebugColor);
}

void AddMacroGroupAABB(bool bEnabled, float3 P0, float3 P1)
{
	if (!bEnabled)
		return;
	AddAABBTWS(P0, P1, float4(1, 0, 1, 1));
}

void AddPageAABB(bool bEnabled, float3 P0, float3 P1, bool bIsPageValid)
{
	if (!bEnabled)
		return;
	const float4 DebugColor = bIsPageValid ? float4(0, 1, 1, 1) : float4(1, 0, 0, 1);
	AddAABBTWS(P0, P1, DebugColor);
}

void AddStepAABB(bool bEnabled, float3 P0, float3 P1, float3 Color)
{
	if (!bEnabled)
		return;
	const float4 DebugColor = float4(Color, 1);
	AddAABBTWS(P0, P1, DebugColor);
}

void AddAABBVolume(bool bEnabled, float3 P0, float3 P1)
{
	if (!bEnabled)
		return;
	const float4 DebugColor = float4(1,1,0, 1);
	AddAABBTWS(P0, P1, DebugColor);
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Jitter functions

float3 GetHairVoxelJitter(uint2 PixelCoord, uint Seed, uint JitterMode)
{
	Seed = JitterMode > 1 ? 0 : Seed;
	return JitterMode > 0 ? float3(
		InterleavedGradientNoise(float2(PixelCoord.xy), float(Seed)),
		InterleavedGradientNoise(float2(PixelCoord.xy), float(Seed * 117.f)),
		InterleavedGradientNoise(float2(PixelCoord.xy), float(Seed * 7901.f))) : float3(0, 0, 0);
}

float4 GetHairVoxelJitter2(uint2 PixelCoord, uint Seed, uint JitterMode)
{
	Seed = JitterMode > 1 ? 0 : Seed;
	const uint2 Seed0 = Rand3DPCG16(int3(PixelCoord, Seed)).xy;
	const uint2 Seed1 = Rand3DPCG16(int3(PixelCoord + 17, Seed)).xy;
	return JitterMode > 0 ? float4(Hammersley16(Seed, 8, Seed0), Hammersley16(Seed, 8, Seed1)) : float4(0, 0, 0, 0);
}

// Use define as this function use global resources not always available where this file is included
#define DEFINE_HAIR_BLUE_NOISE_JITTER_FUNCTION float4 GetHairBlueNoiseJitter(uint2 PixelCoord, uint SampleIndex, uint MaxSampleCount, uint TimeIndex)\
{\
	int2 Offset1 = int2(R2Sequence(SampleIndex) * BlueNoise.Dimensions.xy);\
	int2 Offset2 = int2(R2Sequence(SampleIndex + MaxSampleCount) * BlueNoise.Dimensions.xy);\
	float4 RandomSample;\
	RandomSample.xy = BlueNoiseVec2(PixelCoord + Offset1, TimeIndex);\
	RandomSample.zw = BlueNoiseVec2(PixelCoord + Offset2, TimeIndex);\
	return RandomSample;\
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool ShouldStopTraversal(const FHairTraversalResult InResult, float InHairCountThreshold, bool InUseOpaqueVisibility=true)
{
	// Change this function once we have better result/control over the opaque filtering
	return (InUseOpaqueVisibility && InResult.Visibility == 0) || (InHairCountThreshold != 0 && InResult.HairCount > InHairCountThreshold);
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FHairTraversalSettings
{
	float	DensityScale;
	float	CountThreshold;
	float	DistanceThreshold;
	float	SteppingScale;		// Rate at which the raymarching step increase
	float3	Random;
	float	TanConeAngle;
	float	PixelRadius;
	float	JitterScale;
	float	ForcedMip;
	bool	bUseOpaqueVisibility;
	bool	bDebugEnabled;
	bool	bIsPrimaryRay;
	bool	bCastShadow;	// True of the traversal is for casting (opaque) shadow, i.e., not for hair transmission
};

float GetHairTraversalMaxT()
{
	return 999999;
}

FHairTraversalSettings InitHairTraversalSettings()
{
	FHairTraversalSettings Out;
	Out.DensityScale = 1;
	Out.CountThreshold = 0;
	Out.DistanceThreshold = GetHairTraversalMaxT();
	Out.SteppingScale = 1;
	Out.Random = 0.5f;
	Out.TanConeAngle = 0;
	Out.PixelRadius = 0;
	Out.JitterScale = 1;
	Out.bUseOpaqueVisibility = true;
	Out.bDebugEnabled = false;
	Out.bIsPrimaryRay = false;
	Out.ForcedMip = -1;
	Out.bCastShadow = false;
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Sparse voxel traversal routines
FHairTraversalResult ComputeHairCountVirtualVoxel(
	float3 TranslatedWorldPosition0,
	float3 TranslatedWorldPosition1,
	FVirtualVoxelCommonDesc InCommonDesc,
	FVirtualVoxelNodeDesc InNodeDesc,
	Buffer<uint> InPageIndexBuffer,
	Texture3D<uint> InPageTexture,
	FHairTraversalSettings InSettings)
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_NONE
{
	Error: VOXEL TRAVERSAL TYPE is undefined
}
#endif
#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_LINEAR || VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_LINEAR_MIPMAP
{
	const float MipLevelDistanceScale = 1.0f / InNodeDesc.VoxelWorldSize;

	FHairTraversalResult Out = InitHairTraversalResult();
	if (!InNodeDesc.bIsValid)
	{
		return Out;
	}

	uint3 CurrentPageIndexCoord = 9999;
	bool bIsPageValid = false;
	uint3 PageCoord = 0;

	const float2 HitT = LineBoxIntersect(TranslatedWorldPosition0, TranslatedWorldPosition1, InNodeDesc.TranslatedWorldMinAABB, InNodeDesc.TranslatedWorldMaxAABB);
	if (HitT.x < HitT.y)
	{
		// Count the number of fibers which are within a cylinder defined by the voxel size, 
		// and the distance between the origin and the extent of the volume
		// This assumes that the voxel volume is cubic (i.e. equal dimensions on all sides)
		const float3 O = lerp(TranslatedWorldPosition0, TranslatedWorldPosition1, HitT.xxx);
		const float3 E = lerp(TranslatedWorldPosition0, TranslatedWorldPosition1, HitT.yyy);
		const float OELength = min(length(E - O), InSettings.DistanceThreshold);
		const float3 PagesBoundMax = InNodeDesc.TranslatedWorldMinAABB + InNodeDesc.PageIndexResolution * InNodeDesc.VoxelWorldSize * InCommonDesc.PageResolution;

		#if VOXEL_TRAVERSAL_DEBUG
		FShaderPrintContext Ctx = InitShaderPrintContext(InSettings.bDebugEnabled, uint2(750, 50));
		#endif

		#if VOXEL_TRAVERSAL_DEBUG
		if (InSettings.bDebugEnabled)
		{
			AddLine(true, O, E);
			AddAABBVolume(true, InNodeDesc.TranslatedWorldMinAABB, InNodeDesc.TranslatedWorldMaxAABB);
			AddAABBTWS(InNodeDesc.TranslatedWorldMinAABB, PagesBoundMax, ColorPurple);
		}
		#endif

		// Step according to voxel size
		const float3 D = normalize(E - O) * InNodeDesc.VoxelWorldSize;
		const float MaxStep = float(min(ceil(OELength / InNodeDesc.VoxelWorldSize), 1024.f));
		const float DeltaWorld = OELength / float(MaxStep);
		float StepScale = 1.0f;
		float3 PreviousP = O;
		float ClosestHitT = -1;

		const float3 RandomStepJitter = InSettings.Random.xyz * 2 - 1; // [-1..1]
		for (float StepIt = 0.0f; StepIt < MaxStep; StepIt += StepScale)
		{
			const float SteppingWorldSize = max(DeltaWorld * StepScale, InSettings.TanConeAngle * StepIt * InNodeDesc.VoxelWorldSize);
			const float3 HitP = O + StepIt * D + InSettings.JitterScale * RandomStepJitter * SteppingWorldSize * 0.5f;
			const bool bIsWithinPageBound = all(HitP < PagesBoundMax);

			#if VOXEL_TRAVERSAL_DEBUG
			AddStepAABB(InSettings.bDebugEnabled, PreviousP, HitP, float3(0, 1, 0));
			#endif

			const uint3 VolumeCoord = clamp((HitP - InNodeDesc.TranslatedWorldMinAABB) / InNodeDesc.VoxelWorldSize, 0, InNodeDesc.VirtualResolution - 1);
			const uint3 PageIndexCoord = (VolumeCoord >> InCommonDesc.PageResolutionLog2);

			// Update page index only when needed
			const bool bHasPageIndexChanged = any(PageIndexCoord != CurrentPageIndexCoord);
			if (bHasPageIndexChanged)
			{
				CurrentPageIndexCoord = PageIndexCoord;
				const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, InNodeDesc.PageIndexResolution, InNodeDesc.PageIndexOffset);
				const uint PageIndex = InPageIndexBuffer.Load(LinearPageIndexCoord);

				bIsPageValid = PageIndex != INVALID_VOXEL_PAGE_INDEX;
				{
					PageCoord = IndexToCoord(PageIndex, InCommonDesc.PageCountResolution);
				}
			}

			#if VOXEL_TRAVERSAL_DEBUG
			if (InSettings.bDebugEnabled && bIsPageValid)
			{
				const float3 PageAABBMin = (PageIndexCoord  ) * InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize + InNodeDesc.TranslatedWorldMinAABB;
				const float3 PageAABBMax = (PageIndexCoord+1) * InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize + InNodeDesc.TranslatedWorldMinAABB;
				AddAABBTWS(Ctx, PageAABBMin, PageAABBMax, ColorRed);
			}
			#endif

			if (bIsPageValid && bIsWithinPageBound)
			{
				const uint3 VoxelPageBase   = (PageCoord << InCommonDesc.PageResolutionLog2);
				const uint3 VoxelPageOffset = VolumeCoord - (PageIndexCoord << InCommonDesc.PageResolutionLog2);
				const uint3 VoxelPageCoord  = VoxelPageBase + VoxelPageOffset;

				{
					float HairCountScale = 1;
					float MipLevel = 0.0f;
					#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_LINEAR_MIPMAP
					HairCountScale = SteppingWorldSize * MipLevelDistanceScale;
					MipLevel = log2(SteppingWorldSize * MipLevelDistanceScale);
					#endif
					#if VOXEL_TRAVERSAL_FORCE_MIP_ENABLE
					MipLevel = InSettings.ForcedMip >= 0 ? InSettings.ForcedMip : MipLevel;
					#endif

					const FHairTraversalResult StepResult = GetHairVirtualVoxelDensity(VoxelPageCoord, InPageTexture, uint(MipLevel), InSettings.DensityScale * HairCountScale, InSettings.bCastShadow);
					Acc(Out, StepResult);

					if (InSettings.bIsPrimaryRay && StepResult.HairCount > 0)
					{
						ClosestHitT = 1;
					}

					#if VOXEL_TRAVERSAL_DEBUG
					if (InSettings.bDebugEnabled)
					{
						const float3 WorldOffset = PageIndexCoord * InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize;
						const float MipVoxelWorldSize = InNodeDesc.VoxelWorldSize * InCommonDesc.PageResolution / (InCommonDesc.PageResolution >> uint(MipLevel));

						const uint3 MinCoord = VoxelPageOffset >> uint(MipLevel);
						const uint3 MaxCoord = MinCoord+1;
						const float3 FetchMinAABB = InNodeDesc.TranslatedWorldMinAABB + WorldOffset + MinCoord * MipVoxelWorldSize;
						const float3 FetchMaxAABB = InNodeDesc.TranslatedWorldMinAABB + WorldOffset + MaxCoord * MipVoxelWorldSize;
						AddStepAABB(true, FetchMinAABB, FetchMaxAABB, StepResult.Visibility > 0.5f ? float3(0, 0.5f, 1) : float3(1, 0.5f, 0));
					}
					#endif

					if (ShouldStopTraversal(Out, InSettings.CountThreshold, InSettings.bUseOpaqueVisibility))
					{
						//Out.HitT = length(HitP - O);
						Out.HitT = length(HitP - TranslatedWorldPosition0);
						return Out;
					}
				}
			}
			#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_LINEAR_MIPMAP
			if (InSettings.bIsPrimaryRay && ClosestHitT < 0)
			{
				// Start the mipmap traversal only when we have reach the first hit (primaryview)
			}
			else
			{
				StepScale = min(float(InCommonDesc.PageResolution), StepScale * InSettings.SteppingScale);
			}
			#endif
			PreviousP = HitP;
		}

	}

	return Out;
}
#endif
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_SPARSE_LINEAR || VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_SPARSE_MIPMAP
{
	FHairTraversalResult Out = InitHairTraversalResult();

	int3 CurrentPageIndexCoord = -1;
	bool bIsPageValid = false;
	uint3 PageCoord = 0;

	#if VOXEL_TRAVERSAL_DEBUG
	AddMacroGroupAABB(InSettings.bDebugEnabled, InNodeDesc.TranslatedWorldMinAABB, InNodeDesc.TranslatedWorldMaxAABB);
	AddLine(InSettings.bDebugEnabled, TranslatedWorldPosition0, TranslatedWorldPosition1);
	#endif

	const float2 HitT = LineBoxIntersect(TranslatedWorldPosition0, TranslatedWorldPosition1, InNodeDesc.TranslatedWorldMinAABB, InNodeDesc.TranslatedWorldMaxAABB);
	if (HitT.x < HitT.y)
	{
		// Count the number of fibers which are within a cylinder defined by the voxel size, 
		// and the distance between the origin and the extent of the volume
		// This assumes that the voxel volume is cubic (i.e. equal dimensions on all sides)
		const float3 O = TranslatedWorldPosition0; // lerp(TranslatedWorldPosition, IntersectEndPoint, HitT.xxx);
		const float3 E = TranslatedWorldPosition1; // lerp(TranslatedWorldPosition, IntersectEndPoint, HitT.yyy);
		const float OELength = min(length(E - O), InSettings.DistanceThreshold);

	
		// Init to 1 or -1 depending of the orientation of stepping
		const float3 UNormD = TranslatedWorldPosition1 - TranslatedWorldPosition0;
		const int3 Step = sign(UNormD);

		// Step according to voxel size
		const float3 D = normalize(UNormD) * InNodeDesc.VoxelWorldSize;

		float t = HitT.x;

		// this is slop coeff for each axis: how far we need to move in units of t for each axis
		const float PageWorldSize = InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize;
		const float3 tDelta = Step * PageWorldSize / UNormD;

		// Init to the starting voxel
		int3 PageIndexCoord = -1;
		float3 tMax = 0;
		{
			const float3 HitP = O + HitT.x * UNormD;

			const float Epsilon = 0.000001f;
			const float3 Coords = clamp(
				saturate((HitP - InNodeDesc.TranslatedWorldMinAABB) / (InNodeDesc.TranslatedWorldMaxAABB - InNodeDesc.TranslatedWorldMinAABB)) * InNodeDesc.PageIndexResolution,
				0,
				InNodeDesc.PageIndexResolution - Epsilon);

			const float3 FractCoords = max(Step, 0) - Step * frac(Coords);
			tMax = FractCoords * tDelta;

			PageIndexCoord = clamp(uint3(Coords), uint3(0, 0, 0), InNodeDesc.PageIndexResolution-1);
		}

		// Page stepping is the walking quantity (i.e. number of voxel) for fine ray-marching within a valid page
		float PageStepping = 1;

		const uint LoopCount = 256u;
		for (uint LoopIt = 0; LoopIt < LoopCount; ++LoopIt)
		{
			const bool bIsInside =
				PageIndexCoord.x >= 0 && 
				PageIndexCoord.y >= 0 && 
				PageIndexCoord.z >= 0 &&
				PageIndexCoord.x < int(InNodeDesc.PageIndexResolution.x) && 
				PageIndexCoord.y < int(InNodeDesc.PageIndexResolution.y) && 
				PageIndexCoord.z < int(InNodeDesc.PageIndexResolution.z);
			if (!bIsInside)
			{
				return Out;
			}

			const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, InNodeDesc.PageIndexResolution, InNodeDesc.PageIndexOffset);
			const uint PageIndex = InPageIndexBuffer.Load(LinearPageIndexCoord);

			const bool bIsPageValid = PageIndex != INVALID_VOXEL_PAGE_INDEX;
			#if VOXEL_TRAVERSAL_DEBUG
			{
				const float3 PageMinAABB = PageIndexCoord * InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize + InNodeDesc.TranslatedWorldMinAABB;
				const float3 PageMaxAABB = (PageIndexCoord+float3(1,1,1)) * InCommonDesc.PageResolution * InNodeDesc.VoxelWorldSize + InNodeDesc.TranslatedWorldMinAABB;
				AddPageAABB(InSettings.bDebugEnabled, PageMinAABB, PageMaxAABB, bIsPageValid);
			}
			#endif

			if (bIsPageValid)
			{
				const uint3 PageCoord = IndexToCoord(PageIndex, InCommonDesc.PageCountResolution);

				float3 InnerO = O + t * UNormD;
				const uint MaxInnerStep = InCommonDesc.PageResolution * 1.75f; // ~ Diagonal step count

				#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_SPARSE_LINEAR
				const uint IntPageStepping = 1;
				const uint MipLevel = 0;
				#endif			

				#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_SPARSE_MIPMAP
				const uint IntPageStepping = uint(PageStepping);
				const uint MipLevel = log2(IntPageStepping);
				#endif

				const uint MipPageResolution = InCommonDesc.PageResolution;
				const int3 VoxelPageBase = PageCoord * MipPageResolution;

				for (uint InnerStepIt = 0; InnerStepIt < MaxInnerStep; InnerStepIt += IntPageStepping)
				{
					const float3 InnerHitP = InnerO + D * InnerStepIt;
					const int3 VolumeCoord = PositionToCoordUnclampled(InnerHitP, InNodeDesc.TranslatedWorldMinAABB, InNodeDesc.TranslatedWorldMaxAABB, InNodeDesc.VirtualResolution);
					const int3 VoxelPageOffset = VolumeCoord - PageIndexCoord * InCommonDesc.PageResolution;

					#if VOXEL_TRAVERSAL_DEBUG
					if (InSettings.bDebugEnabled)
					{
						if (InnerStepIt==0)
							AddStepAABB(InSettings.bDebugEnabled, InnerHitP - D * 2, InnerHitP + D*2, float3(0,1,0));
						else
							AddStepAABB(InSettings.bDebugEnabled, InnerHitP, InnerHitP + D, float3(1, 0, 1));
					}
					#endif

					const bool bIsInsideInner =
						VoxelPageOffset.x >= 0 && 
						VoxelPageOffset.y >= 0 && 
						VoxelPageOffset.z >= 0 &&
						VoxelPageOffset.x < int(MipPageResolution) &&
						VoxelPageOffset.y < int(MipPageResolution) &&
						VoxelPageOffset.z < int(MipPageResolution);
					if (!bIsInsideInner)
					{
						break;
					}

					const int3 VoxelPageCoord = VoxelPageBase + VoxelPageOffset;

					#if VOXEL_TRAVERSAL_DEBUG
					{
						const float VoxelExtent = InNodeDesc.VoxelWorldSize * (1 << MipLevel) * 0.5f;
						const float3 P = (PageIndexCoord*InCommonDesc.PageResolution + VoxelPageOffset) * InNodeDesc.VoxelWorldSize + InNodeDesc.TranslatedWorldMinAABB;
						AddStepAABB(InSettings.bDebugEnabled, P - VoxelExtent, P + VoxelExtent, float3(0, 0.5f, 1));
					}
					#endif

					const FHairTraversalResult StepResult = GetHairVirtualVoxelDensity(VoxelPageCoord, InPageTexture, MipLevel, InSettings.DensityScale, InSetttings.bCastShadow);
					Acc(Out, StepResult);

					if (ShouldStopTraversal(Out, InSettings.CountThreshold))
					{
						#if VOXEL_TRAVERSAL_DEBUG
						AddStepAABB(InSettings.bDebugEnabled, InnerHitP - D * 2, InnerHitP + D * 2, float3(1, 0.25f, 0));
						#endif
						Out.HitT = length(InnerHitP - O);
						return Out;
					}
				}
			}

			#if VOXEL_TRAVERSAL_TYPE == VOXEL_TRAVERSAL_SPARSE_MIPMAP
			// Increase the stepping size as we walk away from the start point. 
			// Hypothesis: 
			// * don't increase the stepping too soon as we might miss important closeby details
			// * don't increase too much the stepping other wise we will miss important thing or look too coarse
			PageStepping = clamp(PageStepping += 0.5f, 1.f, 8.f);
			#endif		

			// t is used for defining the intersection point at the entry of a valid page
			t = min(tMax.x, min(tMax.y, tMax.z));

			// Find the next page indx to visit and update the tmax, accordingly
			const float3 Mask = tMax.x < tMax.y ?
				(tMax.x < tMax.z ? float3(1, 0, 0) : float3(0, 0, 1)) :
				(tMax.y < tMax.z ? float3(0, 1, 0) : float3(0, 0, 1));
			PageIndexCoord += Step * Mask;
			tMax += tDelta * Mask;
			
			const float3 NewDelta = tDelta * Mask;
		}
	}


	return Out;
}
#endif
