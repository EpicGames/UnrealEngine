// Copyright Epic Games, Inc. All Rights Reserved.

#define HAIR_STRANDS_PARAMETERS 1 

#include "../Common.ush"
#include "../Common.ush"
#include "../Matrices.ush"
#include "../SceneTextureParameters.ush"
#include "../PositionReconstructionCommon.ush" 
#include "../DeferredShadingCommon.ush"
#include "../ShaderPrint.ush"
#include "HairStrandsAABBCommon.ush"
#include "HairStrandsVisibilityCommon.ush"
#include "HairStrandsClusterCommon.ush"
#include "HairStrandsVertexFactoryCommon.ush"
#include "HairStrandsVoxelPageCommon.ush" 
#include "HairStrandsDeepShadowCommonStruct.ush"

#define GLOBAL_PAGE_COUNTER_INDEX 0
#define GROUPS_PAGE_COUNTER_INDEX 1

///////////////////////////////////////////////////////////////////////////////////////////////////
// Page allocation

#if SHADER_ALLOCATEPAGEINDEX 

float				CPUPageWorldSize;
float				CPUVoxelWorldSize;
uint				bUseCPUVoxelWorldSize;	// When adaptive voxel size is disabled, we use CPU voxel size value
uint				TotalPageIndexCount; 	// This is the max page index count;
uint				PageResolution; 		// Resolution of a page
uint				MacroGroupCount;
uint				IndirectDispatchGroupSize;
uint				bDoesMacroGroupSupportVoxelization;

// For testing parity with CPU version
float4				CPU_TranslatedWorldMinAABB[MAX_HAIR_MACROGROUP_COUNT];
float4				CPU_TranslatedWorldMaxAABB[MAX_HAIR_MACROGROUP_COUNT];
int4				CPU_PageIndexResolution[MAX_HAIR_MACROGROUP_COUNT];
uint				CPU_bUseCPUData;

Buffer<float>		GPUVoxelWorldSize;
Buffer<float>		MacroGroupVoxelSizeBuffer;
Buffer<int>			MacroGroupAABBBuffer;
RWBuffer<int>		MacroGroupVoxelAlignedAABBBuffer;
RWBuffer<uint4>		OutPageIndexResolutionAndOffsetBuffer;
RWBuffer<uint>		OutPageIndexAllocationIndirectBufferArgs;


#if GROUP_SIZE != MAX_HAIR_MACROGROUP_COUNT
#error MAX_HAIR_MACROGROUP_COUNT needs to match MAX_HAIR_MACROGROUP_COUNT
#endif

#define INVALID_OFFSET 0xFFFFFFFF

groupshared uint PageIndexOffsets[MAX_HAIR_MACROGROUP_COUNT];

// This code assume we have less than 32 macro group (which fit into a single CU/SM)
[numthreads(GROUP_SIZE, 1, 1)]
void AllocatePageIndex(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint MacroGroupId = DispatchThreadId.x;

	FHairAABB Bound = InitHairAABB();
	float PageWorldSize = CPUPageWorldSize;
	bool bIsValid = MacroGroupId < MacroGroupCount;
	if (bIsValid)
	{
		const bool bSupportVoxelization = (bDoesMacroGroupSupportVoxelization >> MacroGroupId) & 0x1;
		if (CPU_bUseCPUData > 0)
		{
			Bound.Min = CPU_TranslatedWorldMinAABB[MacroGroupId].xyz;
			Bound.Max = CPU_TranslatedWorldMaxAABB[MacroGroupId].xyz;
		}
		else
		{
			Bound = ReadHairAABB(MacroGroupId, MacroGroupAABBBuffer);
		}

		const float VoxelWorldSize = QuantizeVoxelWorldSize(bUseCPUVoxelWorldSize ? CPUVoxelWorldSize : max(GPUVoxelWorldSize[0], MacroGroupVoxelSizeBuffer[MacroGroupId]));
		PageWorldSize = VoxelWorldSize * PageResolution;

		if (any(Bound.Min > Bound.Max) || !bSupportVoxelization)
		{
			Bound.Min = 0;
			Bound.Max = 0;
			bIsValid = false;
		}
	}

	// Page index allocation
	int3 PageIndexResolution = 0;
	{
		// Snap the max AABB to the voxel size.

		// The contents of MacroGroupAABBBuffer (tight fitting AABBs) and MacroGroupVoxelAlignedAABBBuffer diverge here
		// because the macro group AABBs for voxelization need to be snapped to the voxel page boundary.

		// Allocate enough pages to cover the AABB, where page (0,0,0) origin sit on MinAABB.
		if (bIsValid)
		{
			float3 MacroGroupSize = Bound.Max - Bound.Min;
			if (CPU_bUseCPUData > 0) 
			{ 
				PageIndexResolution = CPU_PageIndexResolution[MacroGroupId].xyz; 
			}
			else
			{
				PageIndexResolution = ceil(MacroGroupSize / PageWorldSize);
			}
			Bound.Max = (PageIndexResolution * PageWorldSize) + Bound.Min; // Snap Bound's Max to page size
		}

		const uint TotalPageIndex = PageIndexResolution.x * PageIndexResolution.y * PageIndexResolution.z;
		PageIndexOffsets[MacroGroupId] = TotalPageIndex;
		GroupMemoryBarrierWithGroupSync();

		// Postfix sum to instance group are always ordered by index
		if (DispatchThreadId.x == 0)
		{
			bool bValidAllocation = true;
			uint PageIndexOffset = 0;
			for (uint LocalMacroGroupId = 0; LocalMacroGroupId < MacroGroupCount; ++LocalMacroGroupId)
			{
				const uint PageCount = PageIndexOffsets[LocalMacroGroupId];
				bValidAllocation = bValidAllocation && (PageIndexOffset + PageCount <= TotalPageIndexCount);
				PageIndexOffsets[LocalMacroGroupId] = bValidAllocation ? PageIndexOffset : INVALID_OFFSET;
				PageIndexOffset += PageCount;
			}
		}
		GroupMemoryBarrierWithGroupSync();

		const uint PageIndexOffset = PageIndexOffsets[MacroGroupId];
		bIsValid = bIsValid && (PageIndexOffset != INVALID_OFFSET);
		if (bIsValid)
		{
			OutPageIndexResolutionAndOffsetBuffer[MacroGroupId] = uint4(PageIndexResolution, PageIndexOffset);
			WriteHairAABB(MacroGroupId, Bound, MacroGroupVoxelAlignedAABBBuffer);
		}
		else
		{
			// Clear all output if the allocation is not valid
			OutPageIndexResolutionAndOffsetBuffer[MacroGroupId] = uint4(0, 0, 0, 0);
			WriteDispatchIndirectArgs(OutPageIndexAllocationIndirectBufferArgs, MacroGroupId, 0, 1, 1);
		}
	}

	if (!bIsValid)
	{
		return;
	}

	// Prepare indirect buffer for doing the actual page index allocation and filling the page index
	{
		const uint AllocatedPageIndexCount = PageIndexResolution.x * PageIndexResolution.y * PageIndexResolution.z;
		WriteDispatchIndirectArgs(OutPageIndexAllocationIndirectBufferArgs, MacroGroupId, DivideAndRoundUp(AllocatedPageIndexCount, IndirectDispatchGroupSize), 1, 1);
	}
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_MARKVALID_PREPARE
uint 			InstanceRegisteredIndex;
uint			ClusterOffset;
uint			ClusterCount;
uint			MacroGroupId;
uint			bUseMacroGroupBoundCPU;
float3			MacroGroupBoundCPU_TranslatedWorldMinAABB;
float3			MacroGroupBoundCPU_TranslatedWorldMaxAABB;
float3			TranslatedWorldOffsetCorrection;

Buffer<int>		GroupAABBsBuffer;
Buffer<int>		ClusterAABBsBuffer;
Buffer<int>		MacroGroupVoxelAlignedAABBBuffer;
Buffer<uint4>	PageIndexResolutionAndOffsetBuffer;

RWBuffer<uint>	OutValidPageIndexBuffer;

// PageIndexBuffer      is sampled with linear coordinate computed from the 3d page coordinate. VALID NODE ARE NOT COMPACTED. It contains the LINEAR PAGE INDEX (to map to the 3d volume).
// PageIndexCoordBuffer is sampled with linear coordinate for allocated nodes. VALID NODE ARE COMPACTED. It contains the 3d page coordinate and ClustedId. Only used for opaque voxel injection.

#if PERMUTATION_USE_CLUSTER
[numthreads(GROUP_SIZE, 1, 1)]
void MarkValid_PrepareCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint ClusterIndex = DispatchThreadId.x;
	if (ClusterIndex >= ClusterCount)
	{
		return;
	}

	const uint BaseClusterIndex = 6 * (ClusterOffset + ClusterIndex);

	FHairAABB ClusterBound;
	ClusterBound.Min.x = float(ClusterAABBsBuffer[BaseClusterIndex + 0]);
	ClusterBound.Min.y = float(ClusterAABBsBuffer[BaseClusterIndex + 1]);
	ClusterBound.Min.z = float(ClusterAABBsBuffer[BaseClusterIndex + 2]);

	ClusterBound.Max.x = float(ClusterAABBsBuffer[BaseClusterIndex + 3]);
	ClusterBound.Max.y = float(ClusterAABBsBuffer[BaseClusterIndex + 4]);
	ClusterBound.Max.z = float(ClusterAABBsBuffer[BaseClusterIndex + 5]);

	if (any(ClusterBound.Min >= ClusterBound.Max))
		return;

	if (any(!IsFinite(ClusterBound.Min)) || any(!IsFinite(ClusterBound.Max)))
		return;

	const uint4 PageIndexResolutionAndOffset = PageIndexResolutionAndOffsetBuffer.Load(MacroGroupId);
	FHairAABB MacroGroupBound = ReadHairAABB(MacroGroupId, MacroGroupVoxelAlignedAABBBuffer);
	const int3 PageIndexResolution = PageIndexResolutionAndOffset.xyz;
	const uint PageIndexOffset = PageIndexResolutionAndOffset.w;


	if (any(MacroGroupBound.Min >= MacroGroupBound.Max))
		return;

	if (any(!IsFinite(MacroGroupBound.Min)) || any(!IsFinite(MacroGroupBound.Max)))
		return;

	uint3 MinCoord = PositionToCoord(ClusterBound.Min, MacroGroupBound.Min, MacroGroupBound.Max, PageIndexResolution);
	uint3 MaxCoord = PositionToCoord(ClusterBound.Max, MacroGroupBound.Min, MacroGroupBound.Max, PageIndexResolution);

	uint3 PageIndexResolutionMinusOne = uint3(PageIndexResolution - 1);
	MinCoord = clamp(MinCoord, uint3(0, 0, 0), PageIndexResolutionMinusOne);
	MaxCoord = clamp(MaxCoord, uint3(0, 0, 0), PageIndexResolutionMinusOne);

	const uint3 CoordResolution = (MaxCoord - MinCoord) + 1;
	const uint ScatterCount = CoordResolution.x * CoordResolution.y * CoordResolution.z;

	// Arbitrary large number (e.g., 100x10x10 pages covered)
	// This acts as guards against degenerated case, where the sim would deformed strands large position making the cluster arbitratry large.
	if (ScatterCount > 10000)
		return;

	if (any(!IsFinite(float3(MinCoord))))
		return;

	if (any(!IsFinite(float3(MaxCoord))))
		return;

	// Find a good sweet spot
	for (uint z = MinCoord.z; z <= MaxCoord.z; ++z)
	{
		for (uint y = MinCoord.y; y <= MaxCoord.y; ++y)
		{
			for (uint x = MinCoord.x; x <= MaxCoord.x; ++x)
			{
				const uint3 PageIndexCoord = uint3(x, y, z);
				const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, PageIndexResolution, PageIndexOffset);
				InterlockedOr(OutValidPageIndexBuffer[LinearPageIndexCoord], 1u);
			}
		}
	}
}
#else // PERMUTATION_USE_CLUSTER
[numthreads(GROUP_SIZE, GROUP_SIZE, GROUP_SIZE)]
void MarkValid_PrepareCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint4 PageIndexResolutionAndOffset = PageIndexResolutionAndOffsetBuffer.Load(MacroGroupId);
	const uint3 PageIndexResolution = PageIndexResolutionAndOffset.xyz;
	const uint PageIndexOffset = PageIndexResolutionAndOffset.w;

	const uint3 Coord = DispatchThreadId;
	if (any(Coord >= PageIndexResolution))
		return;

	FHairAABB MacroGroupBound;
	FHairAABB GroupBound;
	if (bUseMacroGroupBoundCPU)
	{
		MacroGroupBound.Min = MacroGroupBoundCPU_TranslatedWorldMinAABB;
		MacroGroupBound.Max = MacroGroupBoundCPU_TranslatedWorldMaxAABB;

		// HAIR_TODO: Can we have reliable primitive AABB to have tigher bound?
		GroupBound.Min = MacroGroupBoundCPU_TranslatedWorldMinAABB;
		GroupBound.Max = MacroGroupBoundCPU_TranslatedWorldMaxAABB;
	}
	else
	{
		MacroGroupBound = ReadHairAABB(MacroGroupId, MacroGroupVoxelAlignedAABBBuffer);
		GroupBound = ReadHairAABB(InstanceRegisteredIndex, GroupAABBsBuffer);

		// Correct View0 translated world offset to ViewX translated world offset
		GroupBound.Min += TranslatedWorldOffsetCorrection;
		GroupBound.Max += TranslatedWorldOffsetCorrection;
	}

	const uint3 MinCoord = PositionToCoord(GroupBound.Min, MacroGroupBound.Min, MacroGroupBound.Max, PageIndexResolution);
	const uint3 MaxCoord = PositionToCoord(GroupBound.Max, MacroGroupBound.Min, MacroGroupBound.Max, PageIndexResolution);

	if (any(!IsFinite(float3(MinCoord))))
		return;

	if (any(!IsFinite(float3(MaxCoord))))
		return;

	if (all(Coord >= MinCoord) && all(Coord <= MaxCoord))
	{
		const uint3 PageIndexCoord = uint3(Coord.x, Coord.y, Coord.z);
		const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, PageIndexResolution, PageIndexOffset);
		InterlockedOr(OutValidPageIndexBuffer[LinearPageIndexCoord], 1u);
	}
}
#endif // PERMUTATION_USE_CLUSTER
#endif // SHADER_MARKVALID_PREPARE

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_ALLOCATE
uint MacroGroupId;
uint PageCount;
uint CPU_PageIndexCount;
uint CPU_PageIndexOffset;
uint3 CPU_PageIndexResolution;

Buffer<uint4> PageIndexResolutionAndOffsetBuffer;
Buffer<int> IndirectBufferArgs;

RWBuffer<uint> RWPageIndexGlobalCounter;
RWBuffer<uint> RWPageIndexBuffer;
RWBuffer<uint> RWPageToPageIndexBuffer;
RWBuffer<uint4> RWPageIndexCoordBuffer;

groupshared uint LocalCounter;
groupshared uint GroupBase[2];

[numthreads(GROUP_SIZE, 1, 1)]
void AllocateCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (GroupIndex == 0)
	{
		GroupBase[0] = 0;
		GroupBase[1] = 0;
		LocalCounter = 0;
	}
	GroupMemoryBarrierWithGroupSync();

#if PERMUTATION_GPU_DRIVEN == 1
	const uint4 PageIndexResolutionAndOffset = PageIndexResolutionAndOffsetBuffer.Load(MacroGroupId);
	const uint3 PageIndexResolution = PageIndexResolutionAndOffset.xyz;
	const uint  PageIndexOffset = PageIndexResolutionAndOffset.w;
	const uint  PageIndexCount = PageIndexResolution.x * PageIndexResolution.y * PageIndexResolution.z;
#else
	const uint3 PageIndexResolution = CPU_PageIndexResolution;
	const uint  PageIndexOffset = CPU_PageIndexOffset;
	const uint  PageIndexCount = CPU_PageIndexCount;
#endif

	const uint GridIndex = DispatchThreadId.x + PageIndexOffset;
	bool bIsValid = false;
	if (DispatchThreadId.x < PageIndexCount)
	{
		bIsValid = RWPageIndexBuffer[GridIndex] > 0;
	}

	uint Offset = 0;
	if (bIsValid)
	{
		InterlockedAdd(LocalCounter, 1u, Offset);
	}
	GroupMemoryBarrierWithGroupSync();

	if (GroupIndex == 0)
	{
		// * Add page count to global counter for global tacking
		// * Add page count to the group for per group work
		InterlockedAdd(RWPageIndexGlobalCounter[GLOBAL_PAGE_COUNTER_INDEX], LocalCounter, GroupBase[0]);
		InterlockedAdd(RWPageIndexGlobalCounter[GROUPS_PAGE_COUNTER_INDEX + MacroGroupId], LocalCounter, GroupBase[1]);
	}
	GroupMemoryBarrierWithGroupSync();

	if (bIsValid)
	{
		const uint PageIndex0 = GroupBase[0] + Offset; // Global page index
		const uint PageIndex1 = GroupBase[1] + Offset; // Group  page index
		const bool bIsAllocationValid = PageIndex0 < PageCount;

		RWPageIndexBuffer[GridIndex] = bIsAllocationValid ? PageIndex0 : INVALID_VOXEL_PAGE_INDEX;
		if (bIsAllocationValid)
		{
			RWPageToPageIndexBuffer[PageIndex0] = GridIndex;
		}

		// Output the coordinates of the allocated page for indirect dispatch usage
		// If the allocated failed (run out of page), then we mark the IndexCoord with a invalid GroupID
		const uint LinearIndex = DispatchThreadId.x;
		const uint3 PageIndexCoord = IndexToCoord(LinearIndex, PageIndexResolution);
		RWPageIndexCoordBuffer[PageIndexOffset + PageIndex1] = uint4(PageIndexCoord, bIsAllocationValid ? MacroGroupId : INVALID_MACRO_GROUP_ID);
	}
	// Mark page index as invalid
	// Insure that even if write more (due to larger dispatch count that needed), we do not stomp other instance group page index
	else if (DispatchThreadId.x < PageIndexCount)
	{
		RWPageIndexBuffer[GridIndex] = INVALID_VOXEL_PAGE_INDEX;
	}
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_ADDDESC

float3 CPU_TranslatedWorldMinAABB;
uint MacroGroupId;
float3 CPU_TranslatedWorldMaxAABB;
uint CPU_PageIndexOffset;
int3 CPU_PageIndexResolution;
float CPU_VoxelWorldSize;
uint bUseCPUVoxelWorldSize; // When adaptive voxel size is disabled, we use CPU voxel size value

Buffer<float> GPU_VoxelWorldSize;
Buffer<int> MacroGroupVoxelAlignedAABBBuffer;
Buffer<float> MacroGroupVoxelSizeBuffer;
Buffer<uint4> PageIndexResolutionAndOffsetBuffer;
RWStructuredBuffer<FPackedVirtualVoxelNodeDesc> OutNodeDescBuffer;

[numthreads(1, 1, 1)]
void AddDescCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FVirtualVoxelNodeDesc Node;

#if PERMUTATION_GPU_DRIVEN == 1
	const uint4 PageIndexResolutionAndOffset = PageIndexResolutionAndOffsetBuffer.Load(MacroGroupId);
	const FHairAABB TranslatedWorldBound = ReadHairAABB(MacroGroupId, MacroGroupVoxelAlignedAABBBuffer);
	const float VoxelWorldSize = MacroGroupVoxelSizeBuffer[MacroGroupId];

	Node.TranslatedWorldMinAABB = TranslatedWorldBound.Min;
	Node.TranslatedWorldMaxAABB = TranslatedWorldBound.Max;
	Node.PageIndexResolution = PageIndexResolutionAndOffset.xyz;
	Node.PageIndexOffset = PageIndexResolutionAndOffset.w;
	Node.VoxelWorldSize = bUseCPUVoxelWorldSize ? CPU_VoxelWorldSize : max(GPU_VoxelWorldSize[0], VoxelWorldSize);
#else
	Node.TranslatedWorldMinAABB = CPU_TranslatedWorldMinAABB;
	Node.TranslatedWorldMaxAABB = CPU_TranslatedWorldMaxAABB;
	Node.PageIndexResolution = CPU_PageIndexResolution;
	Node.PageIndexOffset = CPU_PageIndexOffset;
	Node.VoxelWorldSize = CPU_VoxelWorldSize;
#endif

	FPackedVirtualVoxelNodeDesc PackedNode = PackVoxelNode(Node);
	OutNodeDescBuffer[MacroGroupId] = PackedNode;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Preapare indirect buffer

#if SHADER_ADDINDIRECTBUFFER

uint PageResolution;
uint MacroGroupCount;
int3 IndirectGroupSize;

Buffer<uint> PageIndexGlobalCounter;
RWBuffer<uint> OutIndirectArgsBuffer;

void WriteArgs(uint WriteIndex, uint AllocatedPageCount)
{
	const uint VoxelCountPerPage = PageResolution * PageResolution * PageResolution;
	const uint DispatchCountX = DivideAndRoundUp(VoxelCountPerPage, IndirectGroupSize.x);
	const uint DispatchCountZ = DivideAndRoundUp(AllocatedPageCount, IndirectGroupSize.z);
	WriteDispatchIndirectArgs(OutIndirectArgsBuffer, WriteIndex, DispatchCountX, 1, DispatchCountZ);
}

[numthreads(GROUP_SIZE, 1, 1)]
void AddIndirectBufferCS(uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Total pages allocated across *all* macro groups
	if (DispatchThreadId.x == 0)
	{
		WriteArgs(GLOBAL_PAGE_COUNTER_INDEX, PageIndexGlobalCounter[GLOBAL_PAGE_COUNTER_INDEX]);
	}

	// Pages allocated for a particular macro group
	const uint MacroGroupId = DispatchThreadId.x;
	if (MacroGroupId < MacroGroupCount)
	{
		WriteArgs(GROUPS_PAGE_COUNTER_INDEX + MacroGroupId, PageIndexGlobalCounter[GROUPS_PAGE_COUNTER_INDEX + MacroGroupId]);
	}
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Indirect clear

#if SHADER_INDPAGECLEAR

Buffer<uint>		PageIndexGlobalCounter;
uint				VirtualVoxelParams_PageResolution;
int3				VirtualVoxelParams_PageCountResolution;
Buffer<uint4>		VirtualVoxelParams_PageIndexCoordBuffer;

RWTexture3D<uint>	OutPageTexture;

[numthreads(GROUP_SIZE_X, 1, GROUP_SIZE_Z)]
void VoxelIndPageClearCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	const uint TotalAllocatedPageCount = PageIndexGlobalCounter[GLOBAL_PAGE_COUNTER_INDEX];
	const uint VoxelCountPerPage = VirtualVoxelParams_PageResolution * VirtualVoxelParams_PageResolution * VirtualVoxelParams_PageResolution;

	const uint LinearVoxelCoord = DispatchThreadId.x;
	const uint AllocatedPageIndex = DispatchThreadId.z;
	if (AllocatedPageIndex < TotalAllocatedPageCount && LinearVoxelCoord < VoxelCountPerPage)
	{
		const uint3 VoxelCoordOffset = IndexToCoord(LinearVoxelCoord, VirtualVoxelParams_PageResolution.xxx);

		const uint PageIndex = AllocatedPageIndex; // PageIndexBuffer is not needed, we already know those tiles are allocated linearly in 3D within OutPageTexture.

		const uint3 PageCoord = IndexToCoord(PageIndex, VirtualVoxelParams_PageCountResolution);
		const int3 VoxelPageBase = PageCoord * VirtualVoxelParams_PageResolution;
		const int3 VoxelCoord = VoxelPageBase + VoxelCoordOffset;

		OutPageTexture[VoxelCoord] = 0;
	}
}

#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_ADAPTIVE_FEEDBACK

#define DEBUG_ENABLE 0

#if DEBUG_ENABLE
#include "../ShaderPrint.ush"
#endif

uint  CPUAllocatedPageCount;
float CPUMinVoxelWorldSize;
float AdaptiveCorrectionThreshold;
float AdaptiveCorrectionSpeed;

Buffer<uint> PageIndexGlobalCounter;
Buffer<float> CurrGPUMinVoxelWorldSize;
RWBuffer<float> NextGPUMinVoxelWorldSize;

float RoundHairVoxelSize(float In)
{
	// Round voxel size to 0.01f to avoid oscillation issue
	return floor(In * 1000.f + 0.5f) * 0.001f;
}

[numthreads(1, 1, 1)]
void FeedbackCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const float CurrVoxelWorldSize = RoundHairVoxelSize(CurrGPUMinVoxelWorldSize[0]);

	// Voxel pages are represent a volume. To derive a better estimate of the ratio by which voxel size needs to be scale, 
	// compute the cubic root of this ratio.
	//
	// AllocatedPage   AllocatedRes^3
	// ------------- = --------------  = VolumeRatio = LinearRatio^3
	//    MaxPage          MaxRes^3

	// Ratio used for predicting voxel size increase
	const uint GPUAllocatedPageCount = PageIndexGlobalCounter[GLOBAL_PAGE_COUNTER_INDEX];
	const float VolumeRatio = float(GPUAllocatedPageCount) / float(CPUAllocatedPageCount);
	const float LinearRatio = pow(VolumeRatio, 1.f / 3.f);

	// Ratio used for predicting voxel size decrease (i.e. when requested allocation fit, 
	// but the voxel size does not match the (more precise) target).
	// In this case, we add a threshold/margin to to the target, so that there is no oscillation.
	const float VolumeRatio_Thres = float(GPUAllocatedPageCount) / float(CPUAllocatedPageCount * AdaptiveCorrectionThreshold);
	const float LinearRatio_Thres = pow(max(VolumeRatio_Thres, 0.f), 1.f / 3.f);

	// If the page pool is not large enough increase voxel size
	float NextVoxelWorldSize = CPUMinVoxelWorldSize;
	if (GPUAllocatedPageCount > CPUAllocatedPageCount)
	{
		//NextVoxelWorldSize = CurrVoxelWorldSize * LinearRatio;
		NextVoxelWorldSize = CurrVoxelWorldSize * LinearRatio_Thres;
	}
	// If the page pool is large enough but the voxel are larger than the requested size decrease voxel size
	else if (GPUAllocatedPageCount < CPUAllocatedPageCount && CurrVoxelWorldSize > CPUMinVoxelWorldSize)
	{
		const float TargetVoxelWorldSize = CurrVoxelWorldSize * LinearRatio_Thres;
		NextVoxelWorldSize = max(CPUMinVoxelWorldSize, lerp(CurrVoxelWorldSize, TargetVoxelWorldSize, AdaptiveCorrectionSpeed));
	}
	//else if (GPUAllocatedPageCount > CPUAllocatedPageCount * AdaptiveCorrectionThreshold)
	//{
	//	const float TargetVoxelWorldSize = CurrVoxelWorldSize * LinearRatio_Thres;
	//	NextVoxelWorldSize = max(CPUMinVoxelWorldSize, lerp(CurrVoxelWorldSize, TargetVoxelWorldSize, AdaptiveCorrectionSpeed));
	//}
	else
	{
		NextVoxelWorldSize = CPUMinVoxelWorldSize;
	}

	// Clamp voxel size
	NextVoxelWorldSize = clamp(RoundHairVoxelSize(NextVoxelWorldSize), HAIR_MIN_VOXEL_SIZE, HAIR_MAX_VOXEL_SIZE);

	// Debug
#if DEBUG_ENABLE
	FFontColor CPUColor = FontEmerald;
	FFontColor GPUColor = FontOrange;
	FFontColor CstColor = FontSilver;

	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(700, 50));

	Print(Context, TEXT(" ------------------------------- "), FontSilver); Newline(Context);
	Print(Context, TEXT("|          Allocations          |"), FontSilver); Newline(Context);
	Print(Context, TEXT(" ------------------------------- "), FontSilver); Newline(Context);

	Print(Context, TEXT("GPU Allocated      "), GPUColor);
	Print(Context, GPUAllocatedPageCount, GPUColor);
	Newline(Context);

	Print(Context, TEXT("CPU Allocated      "), CPUColor);
	Print(Context, CPUAllocatedPageCount, CPUColor);
	Newline(Context);

	Print(Context, TEXT("GPU Curr Min. Size "), GPUColor);
	Print(Context, CurrVoxelWorldSize, GPUColor);
	Newline(Context);

	Print(Context, TEXT("GPU Next Min. Size "), GPUColor);
	Print(Context, NextVoxelWorldSize, GPUColor);
	Newline(Context);

	Print(Context, TEXT("CPU Min. Size      "), CPUColor);
	Print(Context, CPUMinVoxelWorldSize, CPUColor);
	Newline(Context);

	Print(Context, TEXT("Correction Thres.  "), CstColor);
	Print(Context, AdaptiveCorrectionThreshold, CstColor);
	Newline(Context);

	Print(Context, TEXT("Correction Speed   "), CstColor);
	Print(Context, AdaptiveCorrectionSpeed, CstColor);
	Newline(Context);
#endif

	// Update state data
	NextGPUMinVoxelWorldSize[0] = RoundHairVoxelSize(NextVoxelWorldSize);
}

#endif // SHADER_ADAPTIVE_FEEDBACK

///////////////////////////////////////////////////////////////////////////
// Voxel Raster Compute

#if SHADER_RASTERCOMPUTE

uint			MaxRasterCount;
uint			FrameIdMod8;
uint			MacroGroupId;
uint			VertexCount;

uint			VirtualVoxelParams_PageIndexCount;
uint			VirtualVoxelParams_PageResolution;
uint3			VirtualVoxelParams_PageCountResolution;
uint3			VirtualVoxelParams_PageTextureResolution;

Buffer<uint>	VirtualVoxelParams_PageIndexBuffer;
StructuredBuffer<FPackedVirtualVoxelNodeDesc> VirtualVoxelParams_NodeDescBuffer;

RWTexture3D<uint> OutPageTexture;

float CoverageScale;

#define JITTER_ENABLE 0

float3 GetHairVoxelJitter(uint2 PixelCoord, uint Seed)
{
	return float3(
		InterleavedGradientNoise(PixelCoord.xy, Seed),
		InterleavedGradientNoise(PixelCoord.xy, Seed * 117),
		InterleavedGradientNoise(PixelCoord.xy, Seed * 7901));
}

[numthreads(GROUP_SIZE, 1, 1)]
void MainCS(uint2 DispatchThreadID : SV_DispatchThreadID)
{
	uint VertexIndex0 = DispatchThreadID.x;
	uint VertexIndex1 = VertexIndex0 + 1;
	bool bIsValid = VertexIndex0 < VertexCount && VertexIndex1 < VertexCount;
	if (!bIsValid)
		return;

#if PERMUTATION_CULLING == 1
	if (HairStrandsVF_bCullingEnable)
	{
		const uint VertexCountAfterCulling = HairStrandsVF_CullingIndirectBuffer[3];
		uint FetchIndex0 = VertexIndex0;
		uint FetchIndex1 = VertexIndex1;
		bIsValid = FetchIndex0 < VertexCountAfterCulling&& FetchIndex1 < VertexCountAfterCulling;
		if (!bIsValid)
		{
			return;
		}
		FetchIndex1 = min(FetchIndex0 + 1, VertexCountAfterCulling - 1);

		VertexIndex0 = HairStrandsVF_CullingIndexBuffer[FetchIndex0];
		VertexIndex1 = HairStrandsVF_CullingIndexBuffer[FetchIndex1];
	}
#endif

	const float3 PositionOffset = HairStrandsVF_GetHairInstancePositionOffset();

	const FHairControlPoint CP0 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		VertexIndex0,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	if (CP0.Type == HAIR_CONTROLPOINT_END)
		return;

	const FHairControlPoint CP1 = ReadHairControlPoint(
		HairStrandsVF_PositionBuffer,
		VertexIndex1,
		PositionOffset,
		HairStrandsVF_Radius,
		HairStrandsVF_RootScale,
		HairStrandsVF_TipScale);

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxelParams_NodeDescBuffer[MacroGroupId], VirtualVoxelParams_PageResolution);

	const float DiameterToRadius = 0.5f;
	const float3 TranslatedWP0 = mul(float4(CP0.Position, 1), HairStrandsVF_LocalToTranslatedWorldPrimitiveTransform).xyz;
	const float HairCoverage0 = CP0.WorldRadius / max(CP0.WorldRadius, DiameterToRadius * NodeDesc.VoxelWorldSize);

	const float3 TranslatedWP1 = mul(float4(CP1.Position, 1), HairStrandsVF_LocalToTranslatedWorldPrimitiveTransform).xyz;
	const float HairCoverage1 = CP1.WorldRadius / max(CP1.WorldRadius, DiameterToRadius * NodeDesc.VoxelWorldSize);


	// In order to reduce aliasing, we increase the number of steps. This makes the result more comparable to the raster pass.
	const float LineStepMultiplier = 1.5f;

	const float ScaledVoxelWorldSize = NodeDesc.VoxelWorldSize / LineStepMultiplier;

	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution		= VirtualVoxelParams_PageCountResolution;
	CommonDesc.PageTextureResolution	= VirtualVoxelParams_PageTextureResolution;
	CommonDesc.PageResolution			= VirtualVoxelParams_PageResolution;
	CommonDesc.PageResolutionLog2		= VirtualVoxel.PageResolutionLog2;

	// Count the number of fibers which are within a cylinder defined by the voxel size, 
	// and the distance between the origin and the extent of the volume
	// This assumes that the voxel volume is cubic (i.e. equal dimensions on all sides)
	const float3 LineSegment = TranslatedWP1 - TranslatedWP0;
	const float LineLength = length(LineSegment);
	const float3 StepD = normalize(LineSegment) * ScaledVoxelWorldSize;

	// Step according to voxel size
	int3 CurrentPageIndexCoord = -1;
	bool bIsPageValid = false;
	uint3 PageCoord = 0;

	// If we have long segment we could break them into batch (e.g. 8 voxels long), queue them, and indirect dispatch them. 
	// This would make the workload more uniform/coherent. Currently, breaking into smaller batch, does not seems to save a 
	// lot of cost
	#if JITTER_ENABLE
	const float3 Jitter = GetHairVoxelJitter(frac(CP0.Position.xy), FrameIdMod8, VirtualVoxel.JitterMode) * 2 - 1;
	#else
	const float3 Jitter = 0;
	#endif

	const float fMaxStep = LineLength / ScaledVoxelWorldSize;
	const float MaxStep = float(min(ceil(fMaxStep), MaxRasterCount));

	int3 PreviousCoord = -1;
	for (float StepIt = 0.0f; StepIt < MaxStep; ++StepIt)
	{
		const float U = (StepIt + 0.5f) / float(MaxStep);
		const float Radius = lerp(CP0.WorldRadius, CP1.WorldRadius, U);

		const float3 HitP = TranslatedWP0 + StepIt * StepD + Jitter * Radius;
		const int3 VolumeCoord = clamp((HitP - NodeDesc.TranslatedWorldMinAABB) / NodeDesc.VoxelWorldSize, 0, NodeDesc.VirtualResolution-1);
		const int3 PageIndexCoord = VolumeCoord / CommonDesc.PageResolution;

		// Update page index only when needed
		const bool bHasPageIndexChanged = any(PageIndexCoord != CurrentPageIndexCoord);
		if (bHasPageIndexChanged)
		{
			CurrentPageIndexCoord = PageIndexCoord;
			const uint LinearPageIndexCoord = CoordToIndex(PageIndexCoord, NodeDesc.PageIndexResolution, NodeDesc.PageIndexOffset);
			const uint PageIndex = VirtualVoxelParams_PageIndexBuffer.Load(LinearPageIndexCoord);

			bIsPageValid = PageIndex != INVALID_VOXEL_PAGE_INDEX;
			if (bIsPageValid)
			{
				PageCoord = IndexToCoord(PageIndex, CommonDesc.PageCountResolution);
			}
		}

		if (bIsPageValid)
		{
			const int3 VoxelPageBase = PageCoord * CommonDesc.PageResolution;
			const int3 VoxelPageOffset = VolumeCoord - PageIndexCoord * CommonDesc.PageResolution;
			const int3 VoxelPageCoord = VoxelPageBase + VoxelPageOffset;

			// Insure we don't write multiple time within the same voxel.
			// This can happen for small hair segment, where both start & end points could be writtent into the same voxel.
			const bool bRasterize = any(VoxelPageCoord != PreviousCoord);
			if (bRasterize)
			{
				const float VoxelFixPointScale = GetVoxelDensityFixPointScale();
				
				const float HairCoverage = lerp(HairCoverage0, HairCoverage1, U);
				uint RawData = HairCoverage * VoxelFixPointScale * HairStrandsVF_Density * CoverageScale;
				InterlockedAdd(OutPageTexture[VoxelPageCoord], RawData);

				// Groom having raytraced geometry will cast-shadow on opaque geomtry with their RT-geometry, not with their 
				// voxelization. To avoid for doubling/incorrect shadowing, we mark voxel with no-shadow casting flag. 
				// This adds a significant cost when used.
				if (HasHairFlags(HairStrandsVF_Flags, HAIR_FLAGS_RAYTRACING_GEOMETRY))
				{
					InterlockedOr(OutPageTexture[VoxelPageCoord], VOXEL_CAST_NO_SHADOW_MASK);
				}
			}
			PreviousCoord = VoxelPageCoord;
		}
	}
}
#endif // SHADER_RASTERCOMPUTE

///////////////////////////////////////////////////////////////////////////
// Inject opaque surface into voxels

#if SHADER_INJECTOPAQUE_VIRTUALVOXEL
uint MacroGroupId;
float2 SceneDepthResolution;
uint VoxelBiasCount;
uint VoxelMarkCount;
RWTexture3D<uint> OutPageTexture;

uint		 VirtualVoxelParams_PageCount;
uint		 VirtualVoxelParams_PageIndexCount;
uint		 VirtualVoxelParams_PageResolution;
uint3		 VirtualVoxelParams_PageCountResolution;
Buffer<uint> VirtualVoxelParams_PageIndexBuffer;
Buffer<uint> VirtualVoxelParams_AllocatedPageCountBuffer;
Buffer<uint4>VirtualVoxelParams_PageIndexCoordBuffer;
StructuredBuffer<FPackedVirtualVoxelNodeDesc> VirtualVoxelParams_NodeDescBuffer;

[numthreads(GROUP_SIZE_X, 1, GROUP_SIZE_Z)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint LinearVoxelCoord = DispatchThreadId.x;
	const uint AllocatedPageIndex = DispatchThreadId.z;
	const uint VoxelCountPerPage = VirtualVoxelParams_PageResolution.x * VirtualVoxelParams_PageResolution.x * VirtualVoxelParams_PageResolution.x;
	const bool bValidVoxel = AllocatedPageIndex < VirtualVoxelParams_AllocatedPageCountBuffer[0] && LinearVoxelCoord < VirtualVoxelParams_PageCount * VoxelCountPerPage;
	if (!bValidVoxel)
	{
		return;
	}

	uint3 VoxelCoordOffset = IndexToCoord(LinearVoxelCoord, VirtualVoxelParams_PageResolution.xxx);
	VoxelCoordOffset.y = VoxelCoordOffset.y % VirtualVoxelParams_PageResolution.x;
	VoxelCoordOffset.z = VoxelCoordOffset.z % VirtualVoxelParams_PageResolution.x;

	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxelParams_NodeDescBuffer[MacroGroupId], VirtualVoxelParams_PageResolution);

	const uint4 PageIndexCoord = VirtualVoxelParams_PageIndexCoordBuffer[NodeDesc.PageIndexOffset + AllocatedPageIndex];
	const uint  LinearPageIndexCoord = CoordToIndex(PageIndexCoord.xyz, NodeDesc.PageIndexResolution, NodeDesc.PageIndexOffset);

	// PageIndexCoord have invalid .w component if we run out of available page during the allocation
	const bool bIsValid = LinearPageIndexCoord < VirtualVoxelParams_PageIndexCount&& PageIndexCoord.w != INVALID_MACRO_GROUP_ID;
	if (bIsValid)
	{
		const uint PageIndex = VirtualVoxelParams_PageIndexBuffer.Load(LinearPageIndexCoord);
		if (PageIndex != INVALID_VOXEL_PAGE_INDEX)
		{
			const uint3 VoxelCoordBase = PageIndexCoord.xyz * VirtualVoxelParams_PageResolution;
			const uint3 VoxelCoord = VoxelCoordBase + VoxelCoordOffset;
			const float3 TranslatedWorldPosition = VoxelCoord * NodeDesc.VoxelWorldSize + NodeDesc.TranslatedWorldMinAABB;

			float4 ClipPos = mul(float4(TranslatedWorldPosition, 1), PrimaryView.TranslatedWorldToClip);
			ClipPos /= ClipPos.w;
			const float DepthBias = VoxelBiasCount * NodeDesc.VoxelWorldSize;
			const float VoxelDepth = ConvertFromDeviceZ(ClipPos.z) - DepthBias;
			float2 SceneUV = float2(0.5f * (ClipPos.x + 1), 1 - 0.5f * (ClipPos.y + 1));
			SceneUV = ViewportUVToBufferUV(SceneUV);
			const float2 ScenePixelCoord = SceneUV * SceneDepthResolution;

			const bool bIsOnScreen = SceneUV.x >= 0 && SceneUV.x < 1 && SceneUV.y >= 0 && SceneUV.y < 1;
			if (!bIsOnScreen)
				return;

			const float ClosestDepth = ConvertFromDeviceZ(SceneDepthTexture.Load(uint3(ScenePixelCoord, 0)).x);
			const float3 SceneTranslatedWorldPos = ReconstructTranslatedWorldPositionFromDepth(SceneUV, ClosestDepth);

			const bool bIsInVolume =
				SceneTranslatedWorldPos.x >= NodeDesc.TranslatedWorldMinAABB.x && SceneTranslatedWorldPos.x < NodeDesc.TranslatedWorldMaxAABB.x&&
				SceneTranslatedWorldPos.y >= NodeDesc.TranslatedWorldMinAABB.y && SceneTranslatedWorldPos.y < NodeDesc.TranslatedWorldMaxAABB.y&&
				SceneTranslatedWorldPos.z >= NodeDesc.TranslatedWorldMinAABB.z && SceneTranslatedWorldPos.z < NodeDesc.TranslatedWorldMaxAABB.z;

			if (!bIsInVolume)
				return;

			// Inject opaque depth on a thin layer (Dist < DistThreshold) for avoiding weird projection
			if (ClosestDepth < VoxelDepth && abs(ClosestDepth - VoxelDepth) < VoxelMarkCount * NodeDesc.VoxelWorldSize)
			{
				const uint3 VoxelPageIndexCoord = VoxelCoord / VirtualVoxelParams_PageResolution;
				const uint3 VoxelIndexCoordBase = VoxelPageIndexCoord * VirtualVoxelParams_PageResolution;
				const uint3 VoxelPageOffset = VoxelCoord - VoxelIndexCoordBase;

				const uint3 PageCoord = IndexToCoord(PageIndex, VirtualVoxelParams_PageCountResolution);
				const int3 VoxelPageBase = PageCoord * VirtualVoxelParams_PageResolution;
				const int3 VoxelPageCoord = VoxelPageOffset + VoxelPageBase;

				InterlockedOr(OutPageTexture[VoxelPageCoord], VOXEL_OPAQUE_ADD);
			}
		}
	}
}
#endif // SHADER_INJECTOPAQUE_VIRTUALVOXEL

///////////////////////////////////////////////////////////////////////////

#if SHADER_DEPTH_INJECTION
float2 OutputResolution;
uint MacroGroupId;
uint AtlasSlotIndex;

float3 LightDirection;
uint bIsDirectional;
float3 TranslatedLightPosition;

StructuredBuffer<FDeepShadowViewInfo> DeepShadowViewInfoBuffer;

void MainVS(
	uint VertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float3 OutTranslatedWorldPosition : WORLD_POSITION)
{
	const FPackedVirtualVoxelNodeDesc PackedNode = VirtualVoxel.NodeDescBuffer[MacroGroupId];
	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(PackedNode, VirtualVoxel.PageResolution);

	// Move this to an actual vertex/index buffer
	const float3 Min = NodeDesc.TranslatedWorldMinAABB;
	const float3 Max = NodeDesc.TranslatedWorldMaxAABB;

	const float3 Center = (Min + Max) * 0.5f;
	const float3 Extent = (Max - Min) * 0.5f;

	const float3 Position0 = Center + float3(-Extent.x, -Extent.y, -Extent.z);
	const float3 Position1 = Center + float3(+Extent.x, -Extent.y, -Extent.z);
	const float3 Position2 = Center + float3(+Extent.x, +Extent.y, -Extent.z);
	const float3 Position3 = Center + float3(-Extent.x, +Extent.y, -Extent.z);
	const float3 Position4 = Center + float3(-Extent.x, -Extent.y, +Extent.z);
	const float3 Position5 = Center + float3(+Extent.x, -Extent.y, +Extent.z);
	const float3 Position6 = Center + float3(+Extent.x, +Extent.y, +Extent.z);
	const float3 Position7 = Center + float3(-Extent.x, +Extent.y, +Extent.z);

	float3 TranslatedWorldPosition = 0;
	switch (VertexId)
	{
	case  0: TranslatedWorldPosition = Position0; break;
	case  1: TranslatedWorldPosition = Position1; break;
	case  2: TranslatedWorldPosition = Position2; break;
	case  3: TranslatedWorldPosition = Position0; break;
	case  4: TranslatedWorldPosition = Position2; break;
	case  5: TranslatedWorldPosition = Position3; break;

	case  6: TranslatedWorldPosition = Position4; break;
	case  7: TranslatedWorldPosition = Position5; break;
	case  8: TranslatedWorldPosition = Position6; break;
	case  9: TranslatedWorldPosition = Position4; break;
	case 10: TranslatedWorldPosition = Position6; break;
	case 11: TranslatedWorldPosition = Position7; break;

	case 12: TranslatedWorldPosition = Position0; break;
	case 13: TranslatedWorldPosition = Position1; break;
	case 14: TranslatedWorldPosition = Position5; break;
	case 15: TranslatedWorldPosition = Position0; break;
	case 16: TranslatedWorldPosition = Position5; break;
	case 17: TranslatedWorldPosition = Position4; break;

	case 18: TranslatedWorldPosition = Position2; break;
	case 19: TranslatedWorldPosition = Position3; break;
	case 20: TranslatedWorldPosition = Position7; break;
	case 21: TranslatedWorldPosition = Position2; break;
	case 22: TranslatedWorldPosition = Position7; break;
	case 23: TranslatedWorldPosition = Position6; break;

	case 24: TranslatedWorldPosition = Position1; break;
	case 25: TranslatedWorldPosition = Position2; break;
	case 26: TranslatedWorldPosition = Position6; break;
	case 27: TranslatedWorldPosition = Position1; break;
	case 28: TranslatedWorldPosition = Position6; break;
	case 29: TranslatedWorldPosition = Position5; break;

	case 30: TranslatedWorldPosition = Position3; break;
	case 31: TranslatedWorldPosition = Position0; break;
	case 32: TranslatedWorldPosition = Position4; break;
	case 33: TranslatedWorldPosition = Position3; break;
	case 34: TranslatedWorldPosition = Position4; break;
	case 35: TranslatedWorldPosition = Position7; break;
	}

	const FDeepShadowViewInfo DeepShadowViewInfo = DeepShadowViewInfoBuffer[AtlasSlotIndex];
	const float4x4 TranslatedWorldToClipMatrix = DeepShadowViewInfo.TranslatedWorldToClipScaledBiased;

	OutTranslatedWorldPosition = TranslatedWorldPosition;
	OutPosition = mul(float4(TranslatedWorldPosition, 1), TranslatedWorldToClipMatrix);
}

//#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR
#include "HairStrandsVoxelPageTraversal.ush"

void MainPS(
	in float4 InPosition : SV_POSITION,
	in float3 InTranslatedWorldPosition : WORLD_POSITION,
	out float OutDepth : SV_DEPTH)
{
	OutDepth = 0;
	const float2 PixelCoord = InPosition.xy;
	const float2 UV = PixelCoord / float2(OutputResolution); // todo view rect offset

	const float DistanceThreshold = 1000;
	const bool bDebugEnabled = false;
	const float3 SampleRandom = GetHairVoxelJitter(PixelCoord, View.StateFrameIndexMod8, VirtualVoxel.JitterMode);

	const float3 TracingDirection = bIsDirectional ? LightDirection : normalize(InTranslatedWorldPosition - TranslatedLightPosition);
	const float3 TranslatedWP0 = InTranslatedWorldPosition;
	const float3 TranslatedWP1 = InTranslatedWorldPosition + TracingDirection * DistanceThreshold;

	FVirtualVoxelCommonDesc CommonDesc;
	CommonDesc.PageCountResolution = VirtualVoxel.PageCountResolution;
	CommonDesc.PageTextureResolution = VirtualVoxel.PageTextureResolution;
	CommonDesc.PageResolution = VirtualVoxel.PageResolution;
	CommonDesc.PageResolutionLog2 = VirtualVoxel.PageResolutionLog2;

	const FPackedVirtualVoxelNodeDesc PackedNode = VirtualVoxel.NodeDescBuffer[MacroGroupId];
	const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(PackedNode, VirtualVoxel.PageResolution);

	FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
	TraversalSettings.DensityScale = VirtualVoxel.DensityScale;
	TraversalSettings.CountThreshold = 0.9f; // GetOpaqueVoxelValue();
	TraversalSettings.DistanceThreshold = DistanceThreshold;
	TraversalSettings.bDebugEnabled = bDebugEnabled;
	TraversalSettings.SteppingScale = VirtualVoxel.SteppingScale_Shadow;
	TraversalSettings.Random = SampleRandom;
	TraversalSettings.TanConeAngle = 0;
	TraversalSettings.bIsPrimaryRay = true;
	TraversalSettings.bUseOpaqueVisibility = true;
	TraversalSettings.PixelRadius = -1;
	TraversalSettings.ForcedMip = -1;

	const FHairTraversalResult TraversalResult = ComputeHairCountVirtualVoxel(
		TranslatedWP0,
		TranslatedWP1,
		CommonDesc,
		NodeDesc,
		VirtualVoxel.PageIndexBuffer,
		VirtualVoxel.PageTexture,
		TraversalSettings);
	bool bIsValid = TraversalResult.HairCount > 0;

	if (bIsValid)
	{
		const FDeepShadowViewInfo DeepShadowViewInfo = DeepShadowViewInfoBuffer[AtlasSlotIndex];
		const float4x4 TranslatedWorldToClipMatrix = DeepShadowViewInfo.TranslatedWorldToClipScaledBiased;
		const float3 HitP = TranslatedWP0 + normalize(TranslatedWP1 - TranslatedWP0) * TraversalResult.HitT;
		float4 ClipP = mul(float4(HitP, 1), TranslatedWorldToClipMatrix);
		OutDepth = ClipP.z /= ClipP.w;
	}
	else
	{
		discard;
	}
}
#endif //SHADER_DEPTH_INJECTION

///////////////////////////////////////////////////////////////////////////
// Common function for mipmapping voxels

#if SHADER_MIP_VIRTUALVOXEL || SHADER_MIP_INDIRECTARGS
uint ComputeMipDensity(
	const uint RawDensity0,
	const uint RawDensity1,
	const uint RawDensity2,
	const uint RawDensity3,
	const uint RawDensity4,
	const uint RawDensity5,
	const uint RawDensity6,
	const uint RawDensity7)
{
	const float TotalOpaque =
		((RawDensity0 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity1 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity2 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity3 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity4 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity5 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity6 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT) +
		((RawDensity7 & VOXEL_OPAQUE_MASK) >> VOXEL_OPAQUE_SHIFT);
	const uint OutTotalOpaque = uint(clamp(TotalOpaque / 8.f, TotalOpaque > 0 ? 1 : 0, 0x7F)) << VOXEL_OPAQUE_SHIFT;

	// Propagate no shadow casting flag, only if other valid/non-empty voxels are also no-shadow caster
	const bool bHasShadowCaster =
		((RawDensity0 & VOXEL_HAIR_MASK) > 0 && (RawDensity0 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity1 & VOXEL_HAIR_MASK) > 0 && (RawDensity1 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity2 & VOXEL_HAIR_MASK) > 0 && (RawDensity2 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity3 & VOXEL_HAIR_MASK) > 0 && (RawDensity3 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity4 & VOXEL_HAIR_MASK) > 0 && (RawDensity4 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity5 & VOXEL_HAIR_MASK) > 0 && (RawDensity5 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity6 & VOXEL_HAIR_MASK) > 0 && (RawDensity6 & VOXEL_CAST_NO_SHADOW_MASK) == 0) ||
		((RawDensity7 & VOXEL_HAIR_MASK) > 0 && (RawDensity7 & VOXEL_CAST_NO_SHADOW_MASK) == 0);

	uint TotalHair =
		(RawDensity0 & VOXEL_HAIR_MASK) +
		(RawDensity1 & VOXEL_HAIR_MASK) +
		(RawDensity2 & VOXEL_HAIR_MASK) +
		(RawDensity3 & VOXEL_HAIR_MASK) +
		(RawDensity4 & VOXEL_HAIR_MASK) +
		(RawDensity5 & VOXEL_HAIR_MASK) +
		(RawDensity6 & VOXEL_HAIR_MASK) +
		(RawDensity7 & VOXEL_HAIR_MASK);
	const bool bHasData = TotalHair > 0;
	TotalHair /= 8;

	// Insure that if a voxel contains some hair data, its total hair remains > 0 after averaging. 
	// This is important for pruning invalid/empty page later on, to not remove non-empty page (which 
	// could arise due to numerical precision).
	return min(uint(VOXEL_HAIR_MASK), bHasData ? max(TotalHair, 1u) : 0u) | OutTotalOpaque | (bHasShadowCaster ? 0u : VOXEL_CAST_NO_SHADOW_MASK);
}
#endif // SHADER_MIP_VIRTUALVOXEL || SHADER_MIP_INDIRECTARGS

///////////////////////////////////////////////////////////////////////////

#if SHADER_MIP_VIRTUALVOXEL
#include "../MortonCode.ush"

uint bPatchEmptyPage;
int3 PageCountResolution;
uint PageResolution;
uint SourceMip;
uint TargetMip;

Buffer<uint> 	  AllocatedPageCountBuffer;
Texture3D<uint>   InDensityTexture;
RWTexture3D<uint> OutDensityTexture;

uint MortonEncode3(uint3 Voxel)
{
	return MortonCode3(Voxel.x) | MortonCode3(Voxel.y) << 1 | MortonCode3(Voxel.z) << 2;
}

uint3 MortonDecode3(uint Morton)
{
	uint3 Voxel = uint3(ReverseMortonCode3(Morton), ReverseMortonCode3(Morton >> 1), ReverseMortonCode3(Morton >> 2));
	return Voxel;
}

#if PERMUTATION_MIP_AGGREGATE
Buffer<uint>	  PageToPageIndexBuffer;
RWBuffer<uint>	  OutPageIndexBuffer;

RWTexture3D<uint> OutDensityTexture2;
RWTexture3D<uint> OutDensityTexture1;

groupshared uint g_Density4[64][GROUP_SIZE_Z];
groupshared uint g_Density2[8][GROUP_SIZE_Z];
#endif

// GroupSize is [64u,1u,16u]
[numthreads(GROUP_SIZE_X, 1, GROUP_SIZE_Z)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint3 GroupThreadId : SV_GroupThreadID)
{
	const uint  PageIndex = DispatchThreadId.z;
	const uint3 PageCoord = IndexToCoord(PageIndex, PageCountResolution);
	const bool bValid = PageIndex < AllocatedPageCountBuffer[0];
	
	// Dummy initialization for pleasing compiler
	#if PERMUTATION_MIP_AGGREGATE && COMPILER_FXC
	if (GroupThreadId.x < 8 && GroupThreadId.z < GROUP_SIZE_Z)
	{
		g_Density2[GroupThreadId.x][GroupThreadId.z] = 0;
	}
	#endif

	uint Total = 0;
	if (bValid)
	{
		const uint  InPageResolution = PageResolution >> SourceMip;
		const uint OutPageResolution = PageResolution >> TargetMip;
		const uint TotalVoxelPerOutPageResolution = OutPageResolution * OutPageResolution * OutPageResolution;	
		if (DispatchThreadId.x < TotalVoxelPerOutPageResolution)
		{
			const uint VoxelIndex = DispatchThreadId.x;
	
			const uint3 OutVoxelCoordOffset = MortonDecode3(VoxelIndex);
			const uint3  InVoxelCoordOffset = OutVoxelCoordOffset << 1;
		
			const uint3 OutVoxelCoord = PageCoord * OutPageResolution + OutVoxelCoordOffset;
			const uint3  InVoxelCoord = PageCoord * InPageResolution + InVoxelCoordOffset;
		
			const uint3 InVoxelCoord0 = InVoxelCoord;
			const uint3 InVoxelCoord1 = InVoxelCoord0 + uint3(1, 0, 0);
			const uint3 InVoxelCoord2 = InVoxelCoord0 + uint3(0, 1, 0);
			const uint3 InVoxelCoord3 = InVoxelCoord0 + uint3(1, 1, 0);
			const uint3 InVoxelCoord4 = InVoxelCoord0 + uint3(0, 0, 1);
			const uint3 InVoxelCoord5 = InVoxelCoord0 + uint3(1, 0, 1);
			const uint3 InVoxelCoord6 = InVoxelCoord0 + uint3(0, 1, 1);
			const uint3 InVoxelCoord7 = InVoxelCoord0 + uint3(1, 1, 1);
		
			const uint RawDensity0 = InDensityTexture[InVoxelCoord0];
			const uint RawDensity1 = InDensityTexture[InVoxelCoord1];
			const uint RawDensity2 = InDensityTexture[InVoxelCoord2];
			const uint RawDensity3 = InDensityTexture[InVoxelCoord3];
			const uint RawDensity4 = InDensityTexture[InVoxelCoord4];
			const uint RawDensity5 = InDensityTexture[InVoxelCoord5];
			const uint RawDensity6 = InDensityTexture[InVoxelCoord6];
			const uint RawDensity7 = InDensityTexture[InVoxelCoord7];
		
			Total = ComputeMipDensity(
				RawDensity0,
				RawDensity1,
				RawDensity2,
				RawDensity3,
				RawDensity4,
				RawDensity5,
				RawDensity6,
				RawDensity7);
		
			OutDensityTexture[OutVoxelCoord] = Total;
	
			#if PERMUTATION_MIP_AGGREGATE
			// Store 4x4x4 values
			if (GroupThreadId.x < 64)
			{
				uint StoreIndex = MortonEncode3(OutVoxelCoordOffset);
				g_Density4[StoreIndex][GroupThreadId.z] = Total;
			}
			#endif
		}
	}
	
	#if PERMUTATION_MIP_AGGREGATE
	GroupMemoryBarrierWithGroupSync();

	// Target Page Res: 2x2x2 
	if (bValid && GroupThreadId.x < 8)
	{
		const uint Total4 = ComputeMipDensity(
			g_Density4[GroupThreadId.x*8+0][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+1][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+2][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+3][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+4][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+5][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+6][GroupThreadId.z],
			g_Density4[GroupThreadId.x*8+7][GroupThreadId.z]);


		const uint3 StoreVoxel = MortonDecode3(GroupThreadId.x*8)>>1;
		const uint StoreIndex  = MortonEncode3(StoreVoxel); // GroupThreadId.x >> 3;
		g_Density2[StoreIndex][GroupThreadId.z] = Total4;

		const uint OutPageResolution = 2;
		const uint3 OutVoxelCoord = PageCoord * OutPageResolution + StoreVoxel;
		OutDensityTexture2[OutVoxelCoord] = Total4;
	}
	GroupMemoryBarrierWithGroupSync();

	// Target Page Res: 1x1x1 
	if (bValid && GroupThreadId.x < 1)
	{
		const uint Total2 = ComputeMipDensity(
			g_Density2[0][GroupThreadId.z],
			g_Density2[1][GroupThreadId.z],
			g_Density2[2][GroupThreadId.z],
			g_Density2[3][GroupThreadId.z],
			g_Density2[4][GroupThreadId.z],
			g_Density2[5][GroupThreadId.z],
			g_Density2[6][GroupThreadId.z],
			g_Density2[7][GroupThreadId.z]);

		const uint OutPageResolution = 1;
		const uint3 OutVoxelCoord = PageCoord * OutPageResolution + 0;
		OutDensityTexture1[OutVoxelCoord] = Total2;

		// Update the page index with invalid page index if the voxel does not contain any data.
		// This allow to save tracing cost when evaluating the transmittance.

		const bool bIsEmpty =(Total2 & VOXEL_HAIR_MASK) == 0;
		if (bPatchEmptyPage > 0 && bIsEmpty)
		{
			const uint PageIndexOffset = PageToPageIndexBuffer[PageIndex];
			OutPageIndexBuffer[PageIndexOffset] = INVALID_VOXEL_PAGE_INDEX;
		}
	}
	#endif
}
#endif

///////////////////////////////////////////////////////////////////////////

#if SHADER_MIP_INDIRECTARGS
#include "HairStrandsVoxelPageCommon.ush"

uint PageResolution;
uint TargetMipIndex;
int3 DispatchGroupSize;

Buffer<uint>   InIndirectArgs;
RWBuffer<uint> OutIndirectArgs;

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint MacroGroupId = DispatchThreadId.x;

	const uint TargetPageResolution = PageResolution >> TargetMipIndex;
	const uint TotalVoxelCount = TargetPageResolution * TargetPageResolution * TargetPageResolution;
	const uint DispatchX = DivideAndRoundUp(TotalVoxelCount, DispatchGroupSize.x);

	WriteDispatchIndirectArgs(OutIndirectArgs, 0, DispatchX, InIndirectArgs[1], InIndirectArgs[2]);
}

#endif

///////////////////////////////////////////////////////////////////////////

