// Copyright Epic Games, Inc. All Rights Reserved.

#define VF_SUPPORTS_PRIMITIVE_SCENE_DATA 1

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Shared/RenderCurveDefinitions.h"
#include "../Nanite/NaniteHZBCull.ush"
#include "../ColorMap.ush"

#if PERMUTATION_DEBUG
#include "../ShaderPrint.ush"
#endif

// Bin tile
#define BIN_TILE_SIZE 32
#define BIN_TILE_INV_SIZE (1.f / float(BIN_TILE_SIZE))
#define BIN_TILE_SIZE_DIV_AS_SHIFT 5

// Raster tile
#define RASTER_TILE_SIZE 8
#define BIN_RASTER_INV_SIZE (1.f / float(RASTER_TILE_SIZE))
#define RASTER_TILE_SIZE_DIV_AS_SHIFT 3

#define FPackedSegmentType uint4

#if COMPUTESHADER
#ifndef THREADGROUP_SIZE
#error THREADGROUP_SIZE needs to be defined
#endif
#endif

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Debug
#if PERMUTATION_DEBUG

FShaderPrintContext InitShaderPrintContextUnique(bool bActive, uint2 InBaseCoord, uint2 InOffset)
{
	FShaderPrintContext TmpCtx = InitShaderPrintContext(bActive, uint2(0, 0));
	uint UniqueOffset;
	if (bActive)
	{
		SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(TmpCtx, 3) /* Free counter */, 1, UniqueOffset);
	}
	return InitShaderPrintContext(bActive, InBaseCoord + InOffset * UniqueOffset);
}


FShaderPrintContext InitShaderPrintContextAtCursorUnique(uint2 ActiveCoord, uint2 InBaseCoord, uint2 InOffset)
{
	FShaderPrintContext TmpCtx = InitShaderPrintContextAtCursor(ActiveCoord, uint2(0, 0));
	uint UniqueOffset;
	if (TmpCtx.bIsActive)
	{
		SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(TmpCtx, 3) /* Free counter */, 1, UniqueOffset);
	}
	return InitShaderPrintContextAtCursor(ActiveCoord, InBaseCoord + InOffset * UniqueOffset);
}

void PlotCondition(inout FShaderPrintContext Ctx, bool bCondition)
{
	if (bCondition)
	{
		Print(Ctx, TEXT("x "), FontGreen);
	}
	else
	{
		Print(Ctx, TEXT(". "), FontRed);
	}
}

#endif // PERMUTATION_DEBUG

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Pack/unpack

uint  PackR7(float In)   { return uint(saturate(In) * 127.f) & 0x7F; }
float UnpackR7(uint In)  { return (In&0x7F) / 127.f; }
uint  PackR18(float In)  { return uint(saturate(In) * 262143.f) & 0x3FFFF; }
float UnpackR18(uint In) { return (In&0x3FFFF) / 262143.f; }

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Instance
struct FRenderCurveInstanceData
{
	bool bIsValid;
	uint PersistentPrimitiveIndex;
	uint InstanceSceneDataOffset;
	uint ClusterOffset;
	uint ClusterCount;

	float4 TranslatedWorldBoundCenterAndRadius;
	float3 LocalBoundsCenter;
	float3 LocalBoundsExtent;
	float4x4 LocalToTranslatedWorld;
};

FRenderCurveInstanceData GetRenderCurveInstanceData(uint InPrimitiveIndex)
{
	FRenderCurveInstanceData Out = (FRenderCurveInstanceData)0;
	if (InPrimitiveIndex < Scene.RenderCurve.InstanceCount)
	{
		const uint4 Packed = Scene.RenderCurve.RenderCurveInstanceData.Load4(InPrimitiveIndex * RENDERCURVE_PRIMITIVE_DATA_STRIDE_IN_BYTES);
		Out.bIsValid = true;
		Out.PersistentPrimitiveIndex = Packed.x;
		Out.InstanceSceneDataOffset  = Packed.y;
		Out.ClusterOffset = Packed.z;
		Out.ClusterCount  = Packed.w;

		const FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Out.PersistentPrimitiveIndex);
		const FInstanceSceneData InstanceData = GetInstanceSceneDataUnchecked(Out.InstanceSceneDataOffset);

		Out.LocalToTranslatedWorld = DFFastToTranslatedWorld(InstanceData.LocalToWorld, ResolvedView.PreViewTranslation);
		Out.TranslatedWorldBoundCenterAndRadius = float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
		Out.LocalBoundsCenter = InstanceData.LocalBoundsCenter;
		Out.LocalBoundsExtent = InstanceData.LocalBoundsExtent;
	}
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cluster Header
struct FClusterHeader
{
	float3 Center;
	uint CurveCount;
	uint CurvePerCluster;
	uint PointPerCurve;
	float MaxLength;
	float MaxRadius;
	float3 LocalBoundCenter;
	float3 LocalBoundExtent;
};

FClusterHeader GetClusterHeader(uint InClusterIndex)
{
	FClusterHeader Out = (FClusterHeader)0;
	if (InClusterIndex < Scene.RenderCurve.ClusterCount)
	{
		const uint4 Packed0 = Scene.RenderCurve.ClusterData.Load4(InClusterIndex * Scene.RenderCurve.MaxClusterStrideInBytes);
		const uint4 Packed1 = Scene.RenderCurve.ClusterData.Load4(InClusterIndex * Scene.RenderCurve.MaxClusterStrideInBytes + 16u);

		Out.Center           = asfloat(Packed0.xyz);
		Out.CurvePerCluster  = BitFieldExtractU32(Packed0.w, 8, 0);
		Out.PointPerCurve    = BitFieldExtractU32(Packed0.w, 8, 8);
		Out.CurveCount		 = BitFieldExtractU32(Packed0.w, 8, 16);

		Out.MaxLength        = f16tof32(BitFieldExtractU32(Packed1.x, 16,  0));
		Out.MaxRadius        = f16tof32(BitFieldExtractU32(Packed1.x, 16, 16));

		Out.LocalBoundCenter.x = f16tof32(BitFieldExtractU32(Packed1.y, 16,  0));
		Out.LocalBoundCenter.y = f16tof32(BitFieldExtractU32(Packed1.y, 16, 16));
		Out.LocalBoundCenter.z = f16tof32(BitFieldExtractU32(Packed1.z, 16,  0));
		Out.LocalBoundExtent.x = f16tof32(BitFieldExtractU32(Packed1.z, 16, 16));
		Out.LocalBoundExtent.y = f16tof32(BitFieldExtractU32(Packed1.w, 16,  0));
		Out.LocalBoundExtent.z = f16tof32(BitFieldExtractU32(Packed1.w, 16, 16));
		
		Out.LocalBoundCenter += Out.Center;
	}
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Min/Max-Z

struct FMinMaxZ
{
	float MinZ;
	float MaxZ;
	float Range;
	float InvRange;
	float Offset;
	float Scale;
};

FMinMaxZ UnpackMinMaxZ(uint In0, uint In1, float SceneDepthMinZ=0.f)
{
	const float2 In = float2(asfloat(In0), asfloat(In1));

	FMinMaxZ Out;
	Out.MinZ = max(In.x, SceneDepthMinZ);
	Out.MaxZ = In.y;
	Out.Range = Out.MaxZ - Out.MinZ;
	Out.InvRange = 1.f / max(Out.Range, 1e-5f);
	Out.Offset = -Out.MinZ * Out.InvRange;
	Out.Scale = Out.InvRange;
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Cluster Data - Point

struct FCurvePoint
{
	float3 Position;
	float  Radius;
	float  UCoord;
	bool   bValid;
};

FCurvePoint UnpackCurvePoint(uint2 In, float3 InPositionOffset, float InMaxRadius)
{
	FCurvePoint Out = (FCurvePoint)0;
	Out.Position = float3(UnpackFloat2FromUInt(In.x), f16tof32(BitFieldExtractU32(In.y, 16, 0))) + InPositionOffset;
	Out.UCoord	 = UnpackR8(BitFieldExtractU32(In.y, 8, 16));
	Out.Radius	 = UnpackR6(BitFieldExtractU32(In.y, 6, 24)) * InMaxRadius;
	Out.bValid	 = BitFieldExtractU32(In.y, 1, 30);

	return Out;
}

FCurvePoint GetClusterPoint(FClusterHeader Header, uint ClusterIndex, uint CurveIndex, uint PointIndex)
{
	const uint HeaderStride = 32u;
	const uint PointStride  = 8u;
	const uint PointOffsetInBytes = (Header.CurvePerCluster * PointIndex + CurveIndex) * PointStride;
	const uint2 Packed = Scene.RenderCurve.ClusterData.Load2(ClusterIndex * Scene.RenderCurve.MaxClusterStrideInBytes + HeaderStride + PointOffsetInBytes);
	return UnpackCurvePoint(Packed, Header.Center, Header.MaxRadius);
}

///////////////////////////////////////////////////////////////////////////
// Segment

// 64bit segment encoding
// Anchor point top-left tile corner 
// * P0.xy = 7/7bit pos + 18bit depth = 32bit
// * P1.xy = 7/7bit pos + 18bit depth = 32bit
struct FSegment
{	
	float3 P0;
	float3 P1;
	float3 Color;
};

bool ClipSegment(float2 AABBMin, float2 AABBMax, inout float3 P0, inout float3 P1);

FPackedSegmentType PackSegment(uint2 TileCoord, FMinMaxZ MinMaxZ, FSegment In)
{
	const float2 TileP0 = TileCoord     * BIN_TILE_SIZE;
	const float2 TileP1 = (TileCoord+1) * BIN_TILE_SIZE;

	// Clip segment to tile
	ClipSegment(TileP0, TileP1, In.P0, In.P1);

	// Relative to tile corner
	In.P0.xy = (In.P0.xy - TileP0);
	In.P1.xy = (In.P1.xy - TileP0);

	// Normalize in tile space
	const float2 nP0 = In.P0.xy / BIN_TILE_SIZE;
	const float2 nP1 = In.P1.xy / BIN_TILE_SIZE;
	const float nP0z  = (In.P0.z - MinMaxZ.MinZ) * MinMaxZ.InvRange;
	const float nP1z  = (In.P1.z - MinMaxZ.MinZ) * MinMaxZ.InvRange;

	// Quantize
	const uint3 QP0 = uint3(PackR7(nP0.x), PackR7(nP0.y), PackR18(nP0z));
	const uint3 QP1 = uint3(PackR7(nP1.x), PackR7(nP1.y), PackR18(nP1z));

	return FPackedSegmentType(
		QP0.x | (QP0.y<<7) | (QP0.z << 14), 
		QP1.x | (QP1.y<<7) | (QP1.z << 14),
		PackR10G10B10F(In.Color),
		0);
}

FSegment UnpackSegment(uint2 TileCoord, FMinMaxZ MinMaxZ, FPackedSegmentType In)
{
	const float2 TileBase = TileCoord * BIN_TILE_SIZE;

	FSegment Out;
	Out.P0 = float3(
		UnpackR7 (BitFieldExtractU32(In.x,  7,  0)) * BIN_TILE_SIZE + TileBase.x,
		UnpackR7 (BitFieldExtractU32(In.x,  7,  7)) * BIN_TILE_SIZE + TileBase.y,
		UnpackR18(BitFieldExtractU32(In.x, 18, 14)) * MinMaxZ.Range + MinMaxZ.MinZ);
	Out.P1 = float3(
		UnpackR7 (BitFieldExtractU32(In.y,  7,  0)) * BIN_TILE_SIZE + TileBase.x,
		UnpackR7 (BitFieldExtractU32(In.y,  7,  7)) * BIN_TILE_SIZE + TileBase.y,
		UnpackR18(BitFieldExtractU32(In.y, 18, 14)) * MinMaxZ.Range + MinMaxZ.MinZ);
	Out.Color = UnpackR10G10B10F(In.z);
	return Out;
}

///////////////////////////////////////////////////////////////////////////
// DDA helper
 
// TODO: Without setting a limit, we are sometimes caught in an infinite loop even though this should not happen.
#define DDA_MAX_ITERATIONS 256

struct FDDAContext
{
	float2 Coord;
	float2 DeltaDist;
	float2 Step;
	float2 SideDist;
};

FDDAContext DDACreateContext(float2 RayStart, float2 RayDir)
{
	const float2 RayDirRcp = 1.0f / RayDir;

	FDDAContext Context;
	Context.Coord = floor(RayStart);
	Context.DeltaDist = abs(RayDirRcp);
	Context.Step = sign(RayDir);
	Context.SideDist = (Context.Coord - RayStart + max(Context.Step, 0.0f)) * RayDirRcp;

	return Context;
}

void DDAAdvance(inout FDDAContext Context)
{
	if (Context.SideDist.x < Context.SideDist.y)
	{
		Context.SideDist.x += Context.DeltaDist.x;
		Context.Coord.x += Context.Step.x;
	}
	else
	{
		Context.SideDist.y += Context.DeltaDist.y;
		Context.Coord.y += Context.Step.y;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#define USE_SEGMENT_LUT 0
Texture2D<uint2> SegmentLUT;

// Output a 8x8 bitmask of the rastized segment
// * Pos0/Pos1 are in [0..1]
// * Pos0/Pos1 are clipped to border
uint2 GetSegmentBits(Texture2D<float> InSceneDepthTexture, uint2 InCoordOffset, float2 Pos0, float2 Pos1, float PosZ0, float PosZ1, bool bDepthTestEnable)
{
#if USE_SEGMENT_LUT
	Pos0 *= BIN_RASTER_INV_SIZE;
	Pos1 *= BIN_RASTER_INV_SIZE;

	// Sample a 256x256 LUT 4D texture order as follow:
	//    <-----------16------------>
	//     16                       A
	//    [  ] | [  ] | [  ] | [  ] |
	// 16 [  ] | [  ] | [  ] | [  ] |
	//    ---- |----- |----- |----- | 16
	//    [  ] | [  ] | [  ] | [  ] |
	//    [  ] | [  ] | [  ] | [  ] |
	//                              v
	const uint2 iPos0 = min(uint2(Pos0 * 16u) * 16u, 0xFF);
	const uint2 iPos1 = min(uint2(Pos1 * 16u), 0xF);
	const uint2 Coord = iPos0 + iPos1;
	return SegmentLUT[Coord];
#else
	uint2 Out = 0;
	FDDAContext DDAContext = DDACreateContext(Pos0.xy, normalize(Pos1.xy - Pos0.xy));
	const int2 StartCoord = (int2)floor(Pos0.xy);
	const int2 EndCoord = (int2)floor(Pos1.xy);
	for (int DDAIt = 0; DDAIt < 16u; ++DDAIt)
	{
		int2 TileCoord = (int2)floor(DDAContext.Coord);

		TileCoord = clamp(TileCoord, 0, 7);

		// TODO make this more optimal
		// On a simple example this cost 0.7ms
		#if 1
		const float s = clamp(length(TileCoord - StartCoord) / length(EndCoord - StartCoord), 0, 1);
		const float SceneDepth = InSceneDepthTexture.Load(uint3(InCoordOffset + TileCoord, 0));
		const float SegmentDepth = lerp(PosZ0, PosZ1, s);
		const bool bVisible = bDepthTestEnable ? SegmentDepth > SceneDepth : true;
		#else
		const bool bVisible = true;
		#endif
		if (bVisible)
		{
			const uint l = TileCoord.x + TileCoord.y * 8u;
			if (l < 32u)
			{
				Out.x |= 1u << l;
			}
			else
			{
				Out.y |= 1u << (l - 32);
			}
		}

		if (all(TileCoord == EndCoord))
		{
			break;
		}
		DDAAdvance(DDAContext);
	}
	return Out;
#endif
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// HZB
bool HZB(FFrustumCullData FrustumCull)
{
	int4 HZBRect = ResolvedView.ViewRectMinAndSize;  //int4(ResolvedView.ViewRectMinAndSize.xy, ResolvedView.ViewRectMinAndSize.xy + ResolvedView.ViewRectMinAndSize.xy);
	FScreenRect Rect = GetScreenRect( HZBRect, FrustumCull, 4 );
	bool bVisible = true;
	BRANCH
	if(!FrustumCull.bCrossesNearPlane)
	{
		bVisible = IsVisibleHZB(Rect, true);
	}
	return bVisible;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

uint2 PackDepth(float2 In)
{
	return asuint(In);
}

float2 UnpackDepth(uint2 In)
{
	return asfloat(In);
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Projection

float3 NDCToPixelCoord(float4 InDC, uint2 InResolution)
{
	const float3 NDC = InDC.xyz / InDC.w;
	float2 UV = NDC.xy * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	return float3(UV * InResolution, NDC.z);
}

// Line clipping based on "CLIPPING USING HOMOGENEOUS COORDINATES" by Blinn et al.
bool BlinnLineClipping(inout float4 P0, inout float4 P1)
{
	float2 T = float2(0.0f, 1.0f);
	bool bIsRemoved = P0.w < 0.0f && P1.w < 0.0f; // Both points behind near plane

	bool bSign = false;

	UNROLL
	for (uint PlaneIdx = 0; PlaneIdx < 6; ++PlaneIdx)
	{
		// Compute boundary coordinates of both points (w+x, w-x, w+y, w-y, z, w-z)
		bSign = !bSign;
		const uint CompIdx = PlaneIdx / 2;
		const float Sign = bSign ? 1.0f : -1.0f;
		const float WFactor = PlaneIdx != 4 ? 1.0f : 0.0f;
		const float2 BC = WFactor * float2(P0.w, P1.w) + Sign * float2(P0[CompIdx], P1[CompIdx]);

		float Num = BC.x;
		float Denom = BC.x - BC.y;
		bIsRemoved = bIsRemoved || (BC.x < 0.0f && BC.y < 0.0f); // Both points outside the plane
		float Alpha = Num / Denom;
		
		// If the denominator is negative, P0 has a smaller boundary coordinate than P1, so we can assume
		// that P1 is inside the plane (or bIsRemoved is true), so we need to update the alpha for P0.
		// The reverse is true if the denominator is positive.
		if (Denom < 0.0f)
		{
			T.x = max(T.x, Alpha);
		}
		else
		{
			T.y = min(T.y, Alpha);
		}
	}

	if (!bIsRemoved)
	{
		const float4 P0Clipped = lerp(P0, P1, T.x);
		const float4 P1Clipped = lerp(P0, P1, T.y);
		P0 = P0Clipped;
		P1 = P1Clipped;
	}

	return !bIsRemoved;
}

bool InternalClipSegment(float2 AABBMin, float2 AABBMax, float2 P0, float2 P1, out float2 T, out bool2 bClipped)
{
	bClipped = false;
	T = float2(0.0f, 1.0f);
	const bool bP0Outside = any(P0 < AABBMin) || any(P0 > AABBMax);
	const bool bP1Outside = any(P1 < AABBMin) || any(P1 > AABBMax);
	if (!bP0Outside && !bP1Outside)
	{
		return true;
	}

	const float2 Origin = P0;
	const float2 Dir = P1 - P0;
	const float2 RcpDir = 1.0f / Dir;

	const float2 T0 = (AABBMin - Origin) * RcpDir;
	const float2 T1 = (AABBMax - Origin) * RcpDir;

	T.x = max(min(T0.x, T1.x), min(T0.y, T1.y));
	T.y = min(max(T0.x, T1.x), max(T0.y, T1.y));

	// Ray intersects the AABB but the segment is completely outside or no intersection at all.
	if (T.y < 0.0f || T.x > T.y || T.x > 1.f)
	{
		bClipped = true;
		return false;
	}

	if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
	{
		bClipped.x = true;
	}
	if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
	{
		bClipped.y = true;
	}

	return true;
}

bool ClipSegment(float2 AABBMin, float2 AABBMax, inout float3 P0, inout float3 P1)
{
	float2 T = 0;
	bool2 bClipped = false;
	bool bIsValid = InternalClipSegment(AABBMin, AABBMax, P0.xy, P1.xy, T, bClipped);
	
	if (bIsValid)
	{
		const bool bP0Outside = any(P0.xy < AABBMin) || any(P0.xy > AABBMax);
		const bool bP1Outside = any(P1.xy < AABBMin) || any(P1.xy > AABBMax);

		float3 P0New = P0;
		float3 P1New = P1;
		if (bP0Outside && T.x > 0.0f && T.x < 1.0f)
		{
			P0New = lerp(P0, P1, T.x);
			bClipped.x = true;
		}
		if (bP1Outside && T.y > 0.0f && T.y < 1.0f)
		{
			P1New = lerp(P0, P1, T.y);
			bClipped.y = true;
		}
		P0 = P0New;
		P1 = P1New;
	}

	return bIsValid;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
float3 GetCurveColor(uint ClusterIndex, uint CurveIndex)
{
	return ColorMapViridis(float(ClusterIndex * 64 + CurveIndex) / 2048);
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Common parameters

int2 Resolution;

uint BinTileSize;
int2 BinTileRes;
uint NumBinners;

uint RasterTileSize;
int2 RasterTileRes;
uint NumRasterizers;

uint MaxTileDataCount;
uint MaxSegmentDataCount;
uint MaxZBinDataCount;
uint MaxRasterWorkCount;
uint MaxZBinSegmentDataCount;

float MinCoverageThreshold;

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef InstanceCullingCS

RWBuffer<uint> RWVisibleInstanceArgs;
RWStructuredBuffer<uint> RWVisibleInstances;
RWStructuredBuffer<uint> RWMinMaxZ;

[numthreads(64, 1, 1)]
void InstanceCullingCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView(); 

	if (all(DispatchThreadId == 0))
	{
		RWVisibleInstanceArgs[1] = 1;
		RWVisibleInstanceArgs[2] = 1;

		RWMinMaxZ[0] = ~0u;
		RWMinMaxZ[1] =  0u;
	}

	const uint PrimitiveIndex = DispatchThreadId.x;
	const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);

	if (RenderCurveInstanceData.bIsValid)
	{
		// 1. Distance culling
		// TODO?

		// 2. Frustum culling
		const FFrustumCullData FrustumCullData = BoxCullFrustum( 
			RenderCurveInstanceData.LocalBoundsCenter, 
			RenderCurveInstanceData.LocalBoundsExtent, 
			RenderCurveInstanceData.LocalToTranslatedWorld, 
			ResolvedView.TranslatedWorldToClip, 
			ResolvedView.ViewToClip, 
			false /*bIsOrtho*/, 
			true  /*bNearClip*/, 
			false /*bSkipCullFrustum*/);
		bool bIsVisible = FrustumCullData.bIsVisible;

		// 3. HZB culling
		if (bIsHZBValid && bIsVisible)
		{
			bIsVisible = bIsVisible && HZB(FrustumCullData);
		}

		if (bIsVisible)
		{
			uint WriteOffset = 0;
			//WaveInterlockedAddScalarInGroups(RWVisibleInstanceArgs[3], RWVisibleInstanceArgs[0], 64, 1, WriteOffset);
			WaveInterlockedAddScalar_(RWVisibleInstanceArgs[0], 1, WriteOffset);
			RWVisibleInstances[WriteOffset] = PrimitiveIndex;

			#if PERMUTATION_DEBUG
			{
				FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(50, 80 + WriteOffset * 15));
				AddAABBTWS(Ctx, RenderCurveInstanceData.TranslatedWorldBoundCenterAndRadius.xyz - RenderCurveInstanceData.TranslatedWorldBoundCenterAndRadius.www, RenderCurveInstanceData.TranslatedWorldBoundCenterAndRadius.xyz + RenderCurveInstanceData.TranslatedWorldBoundCenterAndRadius.www, ColorYellow);
				AddOBBTWS(Ctx, RenderCurveInstanceData.LocalBoundsCenter - RenderCurveInstanceData.LocalBoundsExtent, RenderCurveInstanceData.LocalBoundsCenter + RenderCurveInstanceData.LocalBoundsExtent, ColorGreen, RenderCurveInstanceData.LocalToTranslatedWorld);
			}
			#endif
		}
	}
}

#endif // InstanceCullingCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef ClusterCullingCS

Buffer<uint> VisibleInstanceArgs;
StructuredBuffer<uint> VisibleInstances;

RWBuffer<uint> RWVisibleClusterArgs;
RWStructuredBuffer<uint2> RWVisibleClusters;
RWStructuredBuffer<uint> RWMinMaxZ;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void ClusterCullingCS(uint3 GroupId : SV_GroupID, uint3 DispatchThreadId : SV_DispatchThreadID, uint LinearThreadIndex : SV_GroupIndex)
{
	ResolvedView = ResolveView(); 

	if (all(DispatchThreadId == 0))
	{
		RWVisibleClusterArgs[1] = 1;
		RWVisibleClusterArgs[2] = 1;
	}

	const uint VisibleInstanceCount = VisibleInstanceArgs[0];
	const uint VisibleInstanceIndex = GroupId.x;
	if (VisibleInstanceIndex >= VisibleInstanceCount)
	{
		return;
	}

	const uint PrimitiveIndex = VisibleInstances[VisibleInstanceIndex];
	const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);

	
	#if PERMUTATION_DEBUG
	if (0)
	{
		//FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(500,50));
		//Print(Ctx, TEXT("CLUSTER CULLING"), FontRed); Newline(Ctx);
		//const float4 ClusterColor = float4(ColorMapMagma(float(ClusterIt) / RenderCurveInstanceData.ClusterCount), 1);
		//AddOBBTWS(Ctx, LocalBoundsCenter - LocalBoundsExtent, LocalBoundsCenter + LocalBoundsExtent, ClusterColor, RenderCurveInstanceData.LocalToTranslatedWorld);
	}
	#endif

	// TODO: change the traversal to be hierarchical and maybe using persistent thread
	for (uint ClusterIt = LinearThreadIndex; ClusterIt < RenderCurveInstanceData.ClusterCount; ClusterIt += THREADGROUP_SIZE)
	{
		const uint ClusterIndex = RenderCurveInstanceData.ClusterOffset + ClusterIt;
		const FClusterHeader ClusterHeader = GetClusterHeader(ClusterIndex);

		if (RenderCurveInstanceData.bIsValid)
		{
			// 1. Distance culling
			// TODO?

			const float3 LocalBoundsCenter = ClusterHeader.LocalBoundCenter;
			const float3 LocalBoundsExtent = ClusterHeader.LocalBoundExtent;

			// 2. Frustum culling
			const FFrustumCullData FrustumCullData = BoxCullFrustum( 
				LocalBoundsCenter, 
				LocalBoundsExtent, 
				RenderCurveInstanceData.LocalToTranslatedWorld, 
				ResolvedView.TranslatedWorldToClip, 
				ResolvedView.ViewToClip, 
				false /*bIsOrtho*/, 
				true  /*bNearClip*/, 
				false /*bSkipCullFrustum*/);
			bool bIsVisible = FrustumCullData.bIsVisible;

			// 3. HZB culling
			if (bIsHZBValid && bIsVisible)
			{
				bIsVisible = bIsVisible && HZB(FrustumCullData);
			}

			if (bIsVisible)
			{
				uint WriteOffset = 0;
				WaveInterlockedAddScalarInGroups(RWVisibleClusterArgs[3], RWVisibleClusterArgs[0], THREADGROUP_SIZE, 1, WriteOffset);
				RWVisibleClusters[WriteOffset] = uint2(PrimitiveIndex, ClusterIndex);
			}

			#if PERMUTATION_DEBUG
			if (0)
			{
				FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(500,500));
				const float4 ClusterColor = float4(ColorMapMagma(float(ClusterIt) / RenderCurveInstanceData.ClusterCount), 1);
				AddOBBTWS(Ctx, LocalBoundsCenter - LocalBoundsExtent, LocalBoundsCenter + LocalBoundsExtent, ClusterColor, RenderCurveInstanceData.LocalToTranslatedWorld);
			}
			#endif

			WaveInterlockedMin(RWMinMaxZ[0], asuint(max(FrustumCullData.RectMin.z, 0.0f)));
			WaveInterlockedMax(RWMinMaxZ[1], asuint(min(FrustumCullData.RectMax.z, 1.0f)));
		}
	}
}

#endif // ClusterCullingCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef SceneTileDepthCS

Texture2D<float> SceneDepthTexture;
RWTexture2D<uint2> OutSceneTileDepthTexture;
groupshared uint group_MinDepth; // (4 bytes)
groupshared uint group_MaxDepth; // (4 bytes)

// 32x32 tile
#define BIN_THREAD_COUNT THREADGROUP_SIZE

#if THREADGROUP_SIZE != (BIN_TILE_SIZE * BIN_TILE_SIZE)
#error Assumes a workgroup size of 1024 threads to cover a maximum tile size of 32x32.
#endif

uint2 LinearTo2D_Bin(uint In)
{
	uint2 Out;
	Out.y = In >> BIN_TILE_SIZE_DIV_AS_SHIFT;
	Out.x = In - Out.y * BIN_TILE_SIZE;
	return Out;
}

[numthreads(THREADGROUP_SIZE, 1, 1)]
void SceneTileDepthCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID) 
{
	if (GroupThreadID == 0)
	{
		group_MinDepth = 0xFFFFFFFF; // Inverse-Z
		group_MaxDepth = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadID < THREADGROUP_SIZE)
	{
		const uint2 PixelCoord = LinearTo2D_Bin(GroupThreadID) + GroupID * BIN_TILE_SIZE;

		if (all(PixelCoord < (uint2)Resolution))
		{
			const float Depth = SceneDepthTexture.Load(uint3(PixelCoord, 0));

			// Compute furthest depth inside this tile
			WaveInterlockedMin(group_MinDepth, asuint(Depth)); // Inverse-Z
			WaveInterlockedMax(group_MaxDepth, asuint(Depth)); // Inverse-Z
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadID == 0)
	{
		OutSceneTileDepthTexture[GroupID] = uint2(group_MinDepth, group_MaxDepth);
	}
}

#endif // SceneTileDepthCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Tile data

// Visibility tile data are stored as:
//   ______________________________________________________________________________________________________________________________________________________________________
// ||                      Tile 0                          ||                      Tile 1                          ||                      Tile 2                          ||
// ||______________________________________________________||______________________________________________________||______________________________________________________||
// ||            |           |             |               ||            |           |             |               ||            |           |             |               ||
// || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex || PrimOffset | PrimCount | (Tile)Coord | MinWriteIndex ||
#define VT_PrimOffset 0
#define VT_PrimCount 1
#define VT_Coord 2
#define VT_MinWriteIndex 3
#define VT_SIZE 4

void StoreTileData(RWStructuredBuffer<uint> OutBuffer, uint Index, uint VTEntry, uint Value)
{
	const uint WriteIndex = Index * VT_SIZE + VTEntry;
	OutBuffer[WriteIndex] = Value;
}
uint LoadTileData(RWStructuredBuffer<uint> OutBuffer, uint Index, uint VTEntry)
{
	const uint ReadIndex = Index * VT_SIZE + VTEntry;
	return OutBuffer[ReadIndex];
}
uint LoadTileData(StructuredBuffer<uint> OutBuffer, uint Index, uint VTEntry)
{
	const uint ReadIndex = Index * VT_SIZE + VTEntry;
	return OutBuffer[ReadIndex];
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#define SEGMENT_COUNT_PER_ALLOC 1024
#if SEGMENT_COUNT_PER_ALLOC != 1024
#error Update binning and compaction code
#endif

#ifdef BinningCS

#define MAX_TILES_TO_ALLOCATE 1024
#define MAX_THREAD_ITERATION_COUNT 4096

StructuredBuffer<uint>    ViewMinMaxZ;
Texture2D<uint2>          SceneTileDepthTexture;
StructuredBuffer<uint2>   VisibleClusters;
StructuredBuffer<uint>    VisibleClustersCount;
RWStructuredBuffer<uint>  VisibleClustersQueue;

RWTexture2DArray<uint>    RWTileSegmentCount;
RWStructuredBuffer<uint>  RWTileData;
RWStructuredBuffer<FPackedSegmentType> RWSegmentData;
RWStructuredBuffer<uint>  RWTileDataAllocatedCount;

groupshared uint group_TilesToAllocate[MAX_TILES_TO_ALLOCATE];
groupshared uint group_TilesToAllocateCount;
groupshared uint group_ClusterIndex;
groupshared uint group_ClusterFetchIndex;
groupshared float4x4 group_LocalToClip;
groupshared FClusterHeader group_ClusterHeader;

// TODO most add a permutation for this
#define PERMUTATION_NUM_POINT_PER_CURVE 16

struct FDebug
{
	#if PERMUTATION_DEBUG
	FShaderPrintContext Ctx;
	#endif
	uint GroupID;
	uint Dummy;
};

// * Each binners fetches work from the visible cluster queue. 
// * Each binner (= a workgroup) loops through all segments of a cluster
// NB there is still potential to use LDS to prevent/reduce contention when adding segments to the binning grid and this may improve perf

void BinCluster(FClusterHeader ClusterHeader, uint BinnerIndex, uint ClusterIndex, uint CurveIt, uint PointIt0, uint PointIt1, float4x4 LocalToClip, uint GroupThreadID, inout FDebug Debug)
{
	// 1. Project segment and clip to screen
	// Each thread of the group is processing a segment of the cluster
	// * GroupThread.x : Curve index
	// * GroupThread.y : Point index
	//
	//     C0 C1 C2 C3 ... C63
	// P0  x  x  x  x      x 
	//     |  |  |  |      |
	// P1  x  x  x  x      x 
	//     |  |  |  |      |
	// P2  x  x  x  x      x 
	//     |  |  |  |      |
	// P3  x  x  x  x      x 
	// ... 	
	#if PERMUTATION_DEBUG
	FShaderPrintContext CtxU = InitShaderPrintContext(true, uint2(500 + CurveIt * 10, 200 + PointIt0 * 10));
	#endif

	bool bValid = false;
	float3 SP0 = 0;
	float3 SP1 = 0;
	if (PointIt1 < ClusterHeader.PointPerCurve)
	{
		const FCurvePoint Point0 = GetClusterPoint(ClusterHeader, ClusterIndex, CurveIt, PointIt0);
		const FCurvePoint Point1 = GetClusterPoint(ClusterHeader, ClusterIndex, CurveIt, PointIt1);
		if (Point0.bValid && Point1.bValid)
		{
			float4 ClipPosition0 = mul(float4(Point0.Position, 1), LocalToClip);
			float4 ClipPosition1 = mul(float4(Point1.Position, 1), LocalToClip);

			// Do clipping in homogenous coordinates
			bValid = true;
			#if 1
			bValid = BlinnLineClipping(ClipPosition0, ClipPosition1); // TODO Is this expensive? Could it be made faster?
			#endif
			SP0 = NDCToPixelCoord(ClipPosition0, Resolution);
			SP1 = NDCToPixelCoord(ClipPosition1, Resolution);

			#if PERMUTATION_DEBUG
			if (0)
			{
				FShaderPrintContext Ctx = InitShaderPrintContext(Debug.GroupID == 0, uint2(0, 0));
				AddLineSS(Ctx, SP0.xy, SP1.xy, ColorGreen, ColorBlue);
			}
			#endif
		}
	}
	#if PERMUTATION_DEBUG
	//PlotCondition(CtxU, bValid);
	#endif
	
	// 2. Reset allocation counter
	if (GroupThreadID == 0)
	{
		group_TilesToAllocateCount = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	// 3. Increment per workgroup per tile counters and add tiles to be allocated
	const float MinZ = min(SP0.z, SP1.z);
	const float2 TileCoord0 = SP0.xy / BIN_TILE_SIZE;
	const float2 TileCoord1 = SP1.xy / BIN_TILE_SIZE;
	if (bValid)
	{
		FDDAContext DDAContext = DDACreateContext(TileCoord0.xy, normalize(TileCoord1.xy - TileCoord0.xy));
		const int2 EndCoord = (int2)floor(TileCoord1.xy);

		for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
		{
			uint DebugInsertMode = 0;
			const int2 TileCoord = (int2)floor(DDAContext.Coord);
			BRANCH
			if (MinZ >= UnpackDepth(SceneTileDepthTexture[TileCoord]).x)  // Inverse-Z
			{
				// Add segment to global counter
				uint OldTileSegmentCount;
				InterlockedAdd(RWTileSegmentCount[uint3(TileCoord, BinnerIndex)], 1, OldTileSegmentCount);
				DebugInsertMode = 1;

				// If global counter reach current span limit (1k segment), queue a span allocation
				BRANCH
				if ((OldTileSegmentCount % 1024) == 0)
				{
					uint WritePos;
					InterlockedAdd(group_TilesToAllocateCount, 1, WritePos);
					if (WritePos < MAX_TILES_TO_ALLOCATE)
					{
						group_TilesToAllocate[WritePos] = PackTileCoord8bits(TileCoord);
						DebugInsertMode = 2;
					}
				}
			}

			#if PERMUTATION_DEBUG
			if (0)
			{
				float4 DebugColor;
				FShaderPrintContext Ctx = InitShaderPrintContext(Debug.GroupID == 0/* && GroupThreadID == 0*/, uint2(0, 0));
				if (DebugInsertMode == 0) DebugColor = ColorRed;
				if (DebugInsertMode == 1) DebugColor = ColorGreen;
				if (DebugInsertMode == 2) DebugColor = ColorYellow;
				AddFilledQuadSS(Ctx, TileCoord * BIN_TILE_SIZE, (TileCoord + 1) * BIN_TILE_SIZE, float4(DebugColor.xyz, 0.01f));
				AddQuadSS(Ctx, TileCoord * BIN_TILE_SIZE, (TileCoord + 1) * BIN_TILE_SIZE, DebugColor);
			}
			#endif

			if (all(TileCoord == EndCoord))
			{
				break;
			}

			DDAAdvance(DDAContext);
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// 4. Allocate new span tiles
	// Segment count has 3 layers:
	// * Tile segment count
	// * Temp segment count
	// * Tile info
	const uint SegmentCountLayerIdx    = BinnerIndex;
	const uint TmpSegmentCountLayerIdx = BinnerIndex + NumBinners * 1;
	const uint TileAllocInfoLayerIdx   = BinnerIndex + NumBinners * 2;

	const uint TilesToAllocateCount = min(MAX_TILES_TO_ALLOCATE, group_TilesToAllocateCount);
	//#if PERMUTATION_DEBUG
	//PrintLineN(Debug.Ctx, TilesToAllocateCount);
	//#endif

	// DEBUG
	for (uint TileIdx = GroupThreadID; TileIdx < TilesToAllocateCount; TileIdx += THREADGROUP_SIZE)
	//for (uint TileIdx = 0; TileIdx < TilesToAllocateCount; TileIdx++)
	{
		//if (GroupThreadID < 1)
		{
			const uint PackedTileCoord = group_TilesToAllocate[TileIdx];
			const uint2 TileCoord = UnpackTileCoord8bits(PackedTileCoord);

			const uint TotalNewWriteCount = RWTileSegmentCount[uint3(TileCoord,    SegmentCountLayerIdx)];
			const uint TotalOldWriteCount = RWTileSegmentCount[uint3(TileCoord, TmpSegmentCountLayerIdx)];

			uint NewTileIndex;
			WaveInterlockedAddScalar_(RWTileDataAllocatedCount[0], 1, NewTileIndex);
			if (NewTileIndex < MaxTileDataCount)
			{
				StoreTileData(RWTileData, NewTileIndex, VT_Coord, PackedTileCoord);
				// Round down the count to the start of the tile and later compare against this to decide which tile to write to.
				StoreTileData(RWTileData, NewTileIndex, VT_MinWriteIndex, TotalNewWriteCount & ~1023u);

				const uint PrevTileIndex = (RWTileSegmentCount[uint3(TileCoord, TileAllocInfoLayerIdx)] & 0xffff);
				if (TotalOldWriteCount > 0)
				{
					StoreTileData(RWTileData, PrevTileIndex, VT_PrimCount, 1024);
				}

				RWTileSegmentCount[uint3(TileCoord, TileAllocInfoLayerIdx)] = (PrevTileIndex << 16) | (NewTileIndex & 0xffff);
			}
		}
	}

	// Visualize allocated tile
	#if PERMUTATION_DEBUG
	if (0)
	//if (Debug.Ctx.bIsActive)
	{
		PrintLineN(Debug.Ctx, Debug.GroupID);
		PrintLineN(Debug.Ctx, TilesToAllocateCount);
		for (uint TileIdx = 0; TileIdx < TilesToAllocateCount; TileIdx++)
		{
			const uint PackedTileCoord = group_TilesToAllocate[TileIdx];
			const uint2 TileCoord = UnpackTileCoord8bits(PackedTileCoord);
			PrintLineN(Debug.Ctx, TileCoord);
			AddQuadSS(Debug.Ctx, TileCoord * BIN_TILE_SIZE, (TileCoord + 1) * BIN_TILE_SIZE, ColorGreen);
		}
	}
	#endif

	GroupMemoryBarrierWithGroupSync();

	// 5. Write segment to tiles
	const FMinMaxZ MinMaxZ = UnpackMinMaxZ(ViewMinMaxZ[0], ViewMinMaxZ[1]);
	if (bValid)
	{
		FDDAContext DDAContext = DDACreateContext(TileCoord0, normalize(TileCoord1 - TileCoord0));
		const int2 EndCoord = (int2)floor(TileCoord1);

		for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
		{
			const int2 TileCoord = (int2)floor(DDAContext.Coord);

			BRANCH
			if (MinZ >= UnpackDepth(SceneTileDepthTexture[TileCoord]).x)  // Inverse-Z
			{
				const uint PackedTiles = RWTileSegmentCount[uint3(TileCoord, TileAllocInfoLayerIdx)];
				const uint CurTile     = BitFieldExtractU32(PackedTiles, 16, 0);
				const uint PrevTile    = BitFieldExtractU32(PackedTiles, 16, 16);

				// Currently we need this to get our write position, but maybe there is a cheaper way to keep track of that?
				uint OldTileSegmentCount;
				InterlockedAdd(RWTileSegmentCount[uint3(TileCoord, TmpSegmentCountLayerIdx)], 1, OldTileSegmentCount);

				const bool bWriteToCurTile = OldTileSegmentCount >= LoadTileData(RWTileData, CurTile, VT_MinWriteIndex);
				const uint LocalWritePos = OldTileSegmentCount % 1024;
				const uint WritePos = (bWriteToCurTile ? CurTile : PrevTile) * 1024 + LocalWritePos;

				if (WritePos < MaxSegmentDataCount)
				{
					FSegment Segment;
					Segment.P0 = SP0;
					Segment.P1 = SP1;
					Segment.Color = GetCurveColor(ClusterIndex, CurveIt);
					RWSegmentData[WritePos] = PackSegment(TileCoord, MinMaxZ, Segment);
				}
				BRANCH
				if (bWriteToCurTile)
				{
					if ((OldTileSegmentCount + 1) == RWTileSegmentCount[uint3(TileCoord, SegmentCountLayerIdx)])
					{
						StoreTileData(RWTileData, CurTile, VT_PrimCount, (OldTileSegmentCount == 1023) ? 1024 : ((OldTileSegmentCount + 1) % 1024));
					}
				}
			}

			if (all(TileCoord == EndCoord))
			{
				break;
			}

			DDAAdvance(DDAContext);
		}
	}
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void BinningCS(uint2 GroupThreadID2D : SV_GroupThreadID, uint GroupThread1D : SV_GroupIndex, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	const uint BinnerIndex = GroupID;
	const uint VisibleClusterCount = VisibleClustersCount[0];

	FDebug Debug;
	#if PERMUTATION_DEBUG
	const bool bDebugEnabled = GroupID <= uint(View.GeneralPurposeTweak) && GroupThread1D == 0;
	//Debug.Ctx = InitShaderPrintContext(bDebugEnabled, uint2(50 + GroupID * 250, 250));
	Debug.Ctx = InitShaderPrintContext(bDebugEnabled, uint2(350 + GroupID * 250, 50));
	Debug.GroupID = GroupID;
	#endif

	// Persistent thread loop for binning the clusters queue
	group_ClusterIndex = 0; 
	group_ClusterFetchIndex = 0; 
	uint IterationIt = 0;
	while (IterationIt < MAX_THREAD_ITERATION_COUNT)
	{
		if (GroupThread1D == 0)
		{
			uint ClusterFetchIndex = 0;
			InterlockedAdd(VisibleClustersQueue[0], 1, ClusterFetchIndex);

			const uint2 VisibleData = VisibleClusters[ClusterFetchIndex];
			const uint  PrimitiveIndex = VisibleData.x;
			const uint  ClusterIndex   = VisibleData.y;
			const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);
			const FClusterHeader ClusterHeader = GetClusterHeader(ClusterIndex);

			group_ClusterFetchIndex = ClusterFetchIndex;
			group_ClusterIndex      = ClusterIndex;
			group_ClusterHeader     = ClusterHeader;
			group_LocalToClip = mul(RenderCurveInstanceData.LocalToTranslatedWorld, ResolvedView.TranslatedWorldToClip);
		}
		GroupMemoryBarrierWithGroupSync();

		if (group_ClusterFetchIndex < VisibleClusterCount)
		{
			const uint CurveIt = GroupThreadID2D.x;
			const uint PointIt0 = GroupThreadID2D.y;
			const uint PointIt1 = PointIt0+1;
			BinCluster(group_ClusterHeader, BinnerIndex, group_ClusterIndex, CurveIt, PointIt0, PointIt1, group_LocalToClip, GroupThread1D, Debug);
		}
		else
		{
			break;
		}

		++IterationIt;
	}
}
#endif // BinningCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FRasterWork
{
	uint2 TileCoord;
	uint ZBinOffset;
	uint ZBinCount;
};

uint2 PackRasterWork(FRasterWork In)
{
	return uint2(In.ZBinOffset, (PackTileCoord8bits(In.TileCoord)<<16u) | (In.ZBinCount & 0xFFFF));
}

FRasterWork UnpackRasterWork(uint2 In)
{
	FRasterWork Out;
	Out.ZBinOffset = In.x;
	Out.ZBinCount = In.y & 0xFFFF;
	Out.TileCoord = UnpackTileCoord8bits(In.y >> 16u);
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FZBin
{
	uint BinZIndex;
	uint PrimOffset;
	uint PrimCount;
};

uint2 PackZBin(FZBin In)
{
	return uint2(In.PrimOffset, (In.BinZIndex & 0x3FF) | (In.PrimCount<<10));
}

FZBin UnpackZBin(uint2 In)
{
	FZBin Out;
	Out.PrimOffset = In.x; 
	Out.PrimCount = In.y >> 10;
	Out.BinZIndex = In.y & 0x3FF;
	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// DEBUG
// Filled ZBin count
// Filled Tile data
// Primitive count in tile
// Occupancy of primitive within tile 8x8/16x16/32x32

#define MAX_SEGMENT_COUNT_PER_ZBIN 1024
#define MAX_ALLOCATED_ZBIN_COUNT 16
#define MAX_TILE_TO_COMPACT 1024

#ifdef CompactionCS

// TODO reduce this?
#define COMPACTION_DEPTH_BUCKET 256

#if COMPACTION_DEPTH_BUCKET > THREADGROUP_SIZE
#error THREADGROUP_SIZE needs to be larger or equal to COMPACTION_DEPTH_BUCKET in order to reset correctly depth bucket values
#endif

Texture2D<uint2> 			SceneTileDepthTexture;
StructuredBuffer<uint> 		ViewMinMaxZ;
Texture2DArray<uint>		TileSegmentCount;
StructuredBuffer<uint>		TileData;
StructuredBuffer<FPackedSegmentType> SegmentData;
StructuredBuffer<uint> 		TileDataAllocatedCount;
							
RWStructuredBuffer<uint> 	RWZBinDataAllocatedCount;
RWStructuredBuffer<uint2>	RWZBinData;
RWStructuredBuffer<uint> 	RWZBinSegmentAllocatedCount;
RWStructuredBuffer<FPackedSegmentType> RWZBinSegmentData;
							
RWStructuredBuffer<uint> 	RWRasterWorkAllocatedCount;
RWStructuredBuffer<uint2> 	RWRasterWork; // Offset & Count + tile coord

groupshared uint group_TilePrimCount;
groupshared uint group_TilePrimOffset;
groupshared uint group_TileToCompactCount;
groupshared uint group_TileToCompact[MAX_TILE_TO_COMPACT];

groupshared uint group_MaxZBinIndex;
groupshared uint group_ZBinOffset[COMPACTION_DEPTH_BUCKET];
groupshared uint group_ZBinCount[COMPACTION_DEPTH_BUCKET];

groupshared uint group_ZBinAllocatedOffset[MAX_ALLOCATED_ZBIN_COUNT];
groupshared uint group_ZBinAllocatedCount[MAX_ALLOCATED_ZBIN_COUNT];

uint GetZBinIndex(float InDepth, FMinMaxZ InMinMaxZ)
{
	// Inverse-Z
	const uint DepthIt = clamp(saturate(InDepth * InMinMaxZ.Scale + InMinMaxZ.Offset) * COMPACTION_DEPTH_BUCKET, 0, COMPACTION_DEPTH_BUCKET - 1);
	return (COMPACTION_DEPTH_BUCKET - 1) - DepthIt;
}

// Launch based on CPU BinTileResX x BinTileResY
// 1 group per screen-tile, 1 threads per bin-tile matching the screen-tile coord
// There can be/are several bins for the same screen area
[numthreads(THREADGROUP_SIZE, 1, 1)]
void CompactionCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThreadID : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	if (GroupThreadID == 0)
	{
		group_TilePrimCount = 0;
		group_TilePrimOffset = 0;
		group_TileToCompactCount = 0;
		//group_ZBinToRefine = 0;
	}

	if (GroupThreadID < COMPACTION_DEPTH_BUCKET)
	{
		group_ZBinOffset[GroupThreadID] = 0;
		group_ZBinCount[GroupThreadID] = 0;
	}

	if (GroupThreadID < MAX_ALLOCATED_ZBIN_COUNT)
	{
		group_ZBinAllocatedOffset[GroupThreadID] = 0;
		group_ZBinAllocatedCount[GroupThreadID] = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	const uint  TileCount = TileDataAllocatedCount[0];
	const uint2 TileCoord = GroupID;
	const uint  TilePackedCoord = PackTileCoord8bits(GroupID); // All thread will process the same tile
	const float SceneMinZ = UnpackDepth(SceneTileDepthTexture.Load(uint3(TileCoord, 0))).x;
	const FMinMaxZ MinMaxZ = UnpackMinMaxZ(ViewMinMaxZ[0], ViewMinMaxZ[1], SceneMinZ);

	// 1. Compute total number of primitives at this tile coordinate
	uint LocalPrimCount = 0;
	for (uint TileIdx = GroupThreadID; TileIdx < TileCount; TileIdx += THREADGROUP_SIZE)
	{
		const uint CurrentTilePackedCoord = LoadTileData(TileData, TileIdx, VT_Coord);

		if (TilePackedCoord == CurrentTilePackedCoord)
		{
			LocalPrimCount += LoadTileData(TileData, TileIdx, VT_PrimCount);
				
			uint WritePos;
			WaveInterlockedAddScalar_(group_TileToCompactCount, 1, WritePos);
			if (WritePos < MAX_TILE_TO_COMPACT)
			{
				group_TileToCompact[WritePos] = TileIdx;
			}
		}
	}

	if (LocalPrimCount > 0)
	{
		WaveInterlockedAdd(group_TilePrimCount, LocalPrimCount);
	}

	GroupMemoryBarrierWithGroupSync();

	const uint TotalPrimCount = group_TilePrimCount;
	if (TotalPrimCount == 0)
	{
		return;
	}

	// 2. Allocate space
	if (GroupThreadID == 0)
	{
		InterlockedAdd(RWZBinSegmentAllocatedCount[0], group_TilePrimCount, group_TilePrimOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	#if PERMUTATION_DEBUG
	FShaderPrintContext Ctx = InitShaderPrintContext(all(GetCursorPos()/BIN_TILE_SIZE == TileCoord) && GroupThreadID == 0, uint2(1500, 200));
	Print(Ctx, TEXT("Compaction"), FontRed); Newline(Ctx);
	PrintLineN(Ctx, TotalPrimCount);
	PrintLineN(Ctx, SceneMinZ);
	PrintLineN(Ctx, MinMaxZ.MinZ);
	PrintLineN(Ctx, MinMaxZ.MaxZ);
	Newline(Ctx);
	#endif

	// 3. Copy PrimIDs to compacted memory
	{
		const uint NumInputTiles = min(group_TileToCompactCount, MAX_TILE_TO_COMPACT);

		// 3.1 First process the LDS list of tiles
		for (uint LDSIdx = 0; LDSIdx < NumInputTiles; ++LDSIdx)
		{
			const uint TileIdx = group_TileToCompact[LDSIdx];
			const uint TilePrimOffset = TileIdx * SEGMENT_COUNT_PER_ALLOC;
			const uint TilePrimCount = LoadTileData(TileData, TileIdx, VT_PrimCount);

			if (GroupThreadID < TilePrimCount)
			{
				const FSegment Segment = UnpackSegment(TileCoord, MinMaxZ, SegmentData[TilePrimOffset + GroupThreadID]);
				const float SegmentMaxZ = max(Segment.P0.z, Segment.P1.z);
				const float SegmentMinZ = min(Segment.P0.z, Segment.P1.z);
				const uint ZBinIndex = GetZBinIndex(SegmentMaxZ, MinMaxZ);
				InterlockedAdd(group_ZBinCount[ZBinIndex], 1);
			}
		}

		GroupMemoryBarrierWithGroupSync();

		// 3.2 Prefix sum of bin count
		// TODO Change to waveops prefixsum
		if (GroupThreadID == 0)
		{
			// 3.2.1 Compute ZBin offset and count
			uint ZBinAllocatedCount = 0;
			{
				uint ZBinAllocatedIndex = 0;
				uint AccSegmentCount = 0;
				uint GlobalOffset = 0;

				group_ZBinAllocatedCount[0]  = 0;
				group_ZBinAllocatedOffset[0] = 0;
				group_MaxZBinIndex = COMPACTION_DEPTH_BUCKET-1;
				for (uint It=0; It < COMPACTION_DEPTH_BUCKET;++It)
				{
					group_ZBinOffset[It] = GlobalOffset;

					const uint CurrentSegmentCount = group_ZBinCount[It];

					if ((AccSegmentCount + CurrentSegmentCount) < MAX_SEGMENT_COUNT_PER_ZBIN)
					{
						// Accumulate segment count
						group_ZBinAllocatedCount[ZBinAllocatedIndex] += CurrentSegmentCount;
					}
					else
					{
						// If we have reach the limit of ZBin we can allocate per tile, mark the max ZBinIndex
						if (ZBinAllocatedIndex+1 >= MAX_ALLOCATED_ZBIN_COUNT)
						{
							group_MaxZBinIndex = It-1u;
							break;
						}

						// New ZBin
						ZBinAllocatedIndex++;

						// Initialize segement offset/count
						group_ZBinAllocatedOffset[ZBinAllocatedIndex] = GlobalOffset;
						group_ZBinAllocatedCount[ZBinAllocatedIndex]  = CurrentSegmentCount;
						AccSegmentCount = 0;

					}
					AccSegmentCount += CurrentSegmentCount;
					GlobalOffset += CurrentSegmentCount;
				}

				ZBinAllocatedCount = ZBinAllocatedIndex + 1;
			}
	
			{
				// 3.2.2 Allocate ZBins
				uint ZBinOffset_Global = 0;
				InterlockedAdd(RWZBinDataAllocatedCount[0], ZBinAllocatedCount, ZBinOffset_Global);

				// 3.2.3 Write ZBins
				if (ZBinOffset_Global+ZBinAllocatedCount < MaxZBinDataCount)
				{
					for (uint It=0; It < ZBinAllocatedCount;++It)
					{
						const uint SegmentOffset = group_TilePrimOffset + group_ZBinAllocatedOffset[It];
						const uint SegmentCount  = group_ZBinAllocatedCount[It];

						#if PERMUTATION_DEBUG
						PrintLineN(Ctx, It);
						PrintLineN(Ctx, SegmentOffset);
						PrintLineN(Ctx, SegmentCount);
						Newline(Ctx);
						#endif

						RWZBinData[ZBinOffset_Global + It] = uint2(SegmentOffset, SegmentCount);
					}
				}

				// 3.2.4 Write raster work
				if (GroupThreadID == 0)
				{
					FRasterWork RasterWork;
					RasterWork.TileCoord = TileCoord;
					RasterWork.ZBinOffset= ZBinOffset_Global;
					RasterWork.ZBinCount = ZBinAllocatedCount;
					
					uint WriteOffset = 0;
					InterlockedAdd(RWRasterWorkAllocatedCount[0], 1, WriteOffset);
					if (WriteOffset < MaxRasterWorkCount)
					{
						RWRasterWork[WriteOffset] = PackRasterWork(RasterWork);
					}
				}
			}
		}

		GroupMemoryBarrierWithGroupSync();

		// 3.3 Clear insertion counter
		if (GroupThreadID < COMPACTION_DEPTH_BUCKET)
		{
			group_ZBinCount[GroupThreadID] = 0;
		}

		GroupMemoryBarrierWithGroupSync();

		// 3.4 Insert primitive into bins
		for (uint LDSIdx = 0; LDSIdx < NumInputTiles; ++LDSIdx)
		{
			const uint TileIdx = group_TileToCompact[LDSIdx];
			const uint TilePrimOffset = TileIdx * 1024;
			const uint TilePrimCount = LoadTileData(TileData, TileIdx, VT_PrimCount);

			if (GroupThreadID < TilePrimCount)
			{
				const FPackedSegmentType PackedSegment = SegmentData[TilePrimOffset + GroupThreadID];
				const FSegment Segment = UnpackSegment(TileCoord, MinMaxZ, PackedSegment);
				const float SegmentNearZ = max(Segment.P0.z, Segment.P1.z); // TODO: always order segment P0 to have nearest Z to avoid loading both points?
				const uint ZBinIndex = GetZBinIndex(SegmentNearZ, MinMaxZ);
				// TODO remapp so that we get ZBin filled up to max

				if (ZBinIndex <= group_MaxZBinIndex)
				{
					uint LocalOffset = 0;
					InterlockedAdd(group_ZBinCount[ZBinIndex], 1, LocalOffset);
					const uint WriteIndex = group_TilePrimOffset + group_ZBinOffset[ZBinIndex] + LocalOffset;
					RWZBinSegmentData[WriteIndex] = PackedSegment;
				}
			}
		}

		// 3.5 Check any remaning tiles (Unlikely?)
		//if (group_TileToCompactCount > 1024)
		//{
		//	for (uint TileIdx = group_MaxLDSTileIdx + 1; TileIdx < TileCount; ++TileIdx)
		//	{
		//		const uint TilePackedCoord = LoadVisTileData(TileData, TileIdx, VT_Coord);
		//		if (PackedCoord == TilePackedCoord)
		//		{
		//			const uint TilePrimOffset = TileIdx * 1024;
		//			const uint TilePrimCount = LoadVisTileData(TileData, TileIdx, VT_PrimCount);
		//
		//			if (GroupThreadID < TilePrimCount)
		//			{
		//				RWZBinSegmentData[CurrentWriteOffset + GroupThreadID] = SegmentData[TilePrimOffset + GroupThreadID];
		//			}
		//
		//			CurrentWriteOffset += TilePrimCount;
		//		}
		//	}
		//}
	}
}

#endif // CompactionCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef SWRasterizerCS

#define MAX_THREAD_ITERATION_COUNT 4096
#define RASTER_TILE_COUNT_2D (BIN_TILE_SIZE / RASTER_TILE_SIZE)           // = 4x4
#define RASTER_TILE_COUNT_1D (RASTER_TILE_COUNT_2D*RASTER_TILE_COUNT_2D)  // = 16

#define MAX_SEGMENT_PER_RASTER_STEP 32

#define WAVE_RASTER 1

// Sanity check
#if RASTER_TILE_COUNT_1D != 16
	#error Update code
#endif
#if RASTER_TILE_COUNT_2D != 4
	#error Update code
#endif
#if MAX_SEGMENT_COUNT_PER_ZBIN > THREADGROUP_SIZE
	#error MAX_SEGMENT_COUNT_PER_ZBIN needs to be smaller than THREADGROUP_SIZE to ensure all segment of a given ZBin could be loaded in one iteration
#endif
#if (MAX_SEGMENT_PER_RASTER_STEP * RASTER_TILE_COUNT_1D) > THREADGROUP_SIZE
	#error MAX_SEGMENT_PER_RASTER_STEP is too large, clearing won't be done in a single step
#endif
#if MAX_SEGMENT_PER_RASTER_STEP != 32
	#error  Update code, as waveops 32 are used for segment rasterization
#endif

StructuredBuffer<uint> 	ViewMinMaxZ;
							
StructuredBuffer<uint> 	ZBinDataAllocatedCount;
StructuredBuffer<uint2>	ZBinData;
StructuredBuffer<uint> 	ZBinSegmentAllocatedCount;
StructuredBuffer<FPackedSegmentType> ZBinSegmentData;
							
StructuredBuffer<uint> 	RasterWorkAllocatedCount;
StructuredBuffer<uint2> RasterWork; // Offset & Count + tile coord
RWStructuredBuffer<uint>RasterWorkQueue;

Texture2D<uint2> 		SceneTileDepthTexture;
Texture2D<float> 		SceneDepthTexture;
RWTexture2D<float4> 	OutputTexture;

groupshared uint group_WorkFetchIndex;
groupshared uint group_Valid;
groupshared FRasterWork group_Work;

groupshared FPackedSegmentType group_PackedSegments[MAX_SEGMENT_COUNT_PER_ZBIN];
#if WAVE_RASTER
#define THREADGROUP_WAVE_COUNT (THREADGROUP_SIZE / 32)
#if THREADGROUP_WAVE_COUNT * MAX_SEGMENT_PER_RASTER_STEP > THREADGROUP_SIZE
#error Update code as we expect a certain number of wave size to rasterize the segment
#endif
groupshared uint2 group_SegmentsBits[THREADGROUP_SIZE];
groupshared float group_SegmentsColor[THREADGROUP_SIZE];
groupshared uint group_CompletedWaves[RASTER_TILE_COUNT_1D * 2]; // Wave
#else
groupshared uint2 group_SegmentsBits[RASTER_TILE_COUNT_1D][MAX_SEGMENT_PER_RASTER_STEP]; 	// 8x8 bit mask per segments. 32 segments
#endif
//groupshared float group_Coverage[RASTER_TILE_COUNT_1D][RASTER_TILE_SIZE][RASTER_TILE_SIZE]; // 16 tiles of 8x8 - Needs to be reduce -> 8bit compaction for coverage?
//groupshared float3 group_Color[RASTER_TILE_COUNT_1D][RASTER_TILE_SIZE][RASTER_TILE_SIZE];

groupshared uint group_SceneMaxZ[RASTER_TILE_COUNT_1D];

#define COVERAGE_CULLING 1

// 8x256 = 2048
// 8x32  = 256  -> x16  =4096
// 8x8   = 64   -> x16  =1024
// ------------
// LDS 2.5k per group
// x16 = 32k 
struct FTileThreadCoord
{
	uint2 Tile;
	uint2 Thread;

	uint Tile1d;
	uint Thread1d;
};

struct FOutputCoord
{
	uint2 PixelCoord;
};

struct FCoord
{
	FTileThreadCoord Bin;
	FTileThreadCoord Raster;
	FOutputCoord Out;
};

void DrawBitLine(RWTexture2D<float4> Out, uint2 OutResolution, uint2 OutBaseCoord, uint2 In)
{	
	for (uint y=0;y<8; ++y)
	for (uint x=0;x<8; ++x)
	{
		const uint l = x + y * 8;
		const uint b = l<32u ? ((In.x>>l)&0x1) : ((In.y>>(l-32u))&0x1);
		if (b > 0)
		{
			const uint2 OutCoord = OutBaseCoord + uint2(x, y);
			if (all(OutCoord < OutResolution))
			Out[OutCoord] = float4(0,1,0,1);
		}
	}
}
#if PERMUTATION_DEBUG
void PlotRasterTileCoverage(inout FShaderPrintContext Ctx, uint RasterTile1d)
{
	//const float RasterTileMinCoverage = asfloat(group_RasterTileMinCoverage[RasterTile1d]);
	//PrintLineN(Ctx, RasterTileMinCoverage);
	//Newline(Ctx);
	//for (uint y = 0; y < 8; ++y)
	//{
	//	for (uint x = 0; x < 8; ++x)
	//	{
	//		const float Cov = group_Coverage[RasterTile1d][x][y];
	//		if (Cov > 0)
	//		Print(Ctx, TEXT("x "), FontGreen);
	//		else
	//		Print(Ctx, TEXT(". "), FontWhite);
	//	}
	//	Newline(Ctx);
	//}
}

void PlotWorkInfo(inout FShaderPrintContext Ctx, FCoord InCoord)
{
	PrintLineN(Ctx, group_Work.TileCoord);
	PrintLineN(Ctx, group_Work.ZBinOffset);
	//PrintLineN(Ctx, group_Work.ZBinCount);
	Print(Ctx, TEXT("ZBinCount :"), FontRed); Print(Ctx, group_Work.ZBinCount, FontRed); Newline(Ctx);
	Newline(Ctx);

	PrintLineN(Ctx, InCoord.Bin.Tile);
	PrintLineN(Ctx, InCoord.Bin.Thread);
	PrintLineN(Ctx, InCoord.Raster.Tile);
	PrintLineN(Ctx, InCoord.Raster.Thread);
	PrintLineN(Ctx, InCoord.Out.PixelCoord);
	Newline(Ctx);
}

void PlotRasterTileAABB(inout FShaderPrintContext Ctx, FCoord InCoord)
{
	const float2 AABBMin = InCoord.Bin.Tile * BIN_TILE_SIZE +  InCoord.Raster.Tile    * RASTER_TILE_SIZE;
	const float2 AABBMax = InCoord.Bin.Tile * BIN_TILE_SIZE + (InCoord.Raster.Tile+1) * RASTER_TILE_SIZE;
	AddQuadSS(Ctx, AABBMin, AABBMax, ColorRed);
}

void PlotUnclippedSegment(FCoord InCoord, uint SegIt, FMinMaxZ MinMaxZ)
{
	const FSegment Segment = UnpackSegment(InCoord.Bin.Tile, MinMaxZ, group_PackedSegments[SegIt]);
	FShaderPrintContext CtxD = InitShaderPrintContext(true, 0);
	AddLineSS(CtxD, Segment.P0.xy, Segment.P1.xy, ColorPurple);
}

FShaderPrintContext GetShaderPrintContextPerRasterThread(FCoord Coord)
{
	const float2 AABBMin = Coord.Bin.Tile * BIN_TILE_SIZE +  Coord.Raster.Tile    * RASTER_TILE_SIZE;
	const float2 AABBMax = Coord.Bin.Tile * BIN_TILE_SIZE + (Coord.Raster.Tile+1) * RASTER_TILE_SIZE;
	const uint2 CursorCoord = GetCursorPos();
	return InitShaderPrintContext(all(CursorCoord >= AABBMin) && all(CursorCoord < AABBMax), uint2(450, 450) + Coord.Raster.Thread * 20);
}
#endif

[numthreads(THREADGROUP_SIZE, 1, 1)]
void SWRasterizerCS(uint DispatchThreadID : SV_DispatchThreadID, uint GroupThread1D : SV_GroupIndex, uint GroupID : SV_GroupID)
{
	ResolvedView = ResolveView();

	// Compute alll the coordinate (Bin/Raster/Output - Tile/Thread/Thread1d)
	FCoord Coord;
	{
		// Use harcoded value for clarity. Ensure the code is coherent
		#if RASTER_TILE_COUNT_2D != 4u
		#error Update tile code
		#endif

		const uint2 ThreadBlock8x256       = uint2(GroupThread1D % RASTER_TILE_SIZE, GroupThread1D / RASTER_TILE_SIZE);
		const uint2 ThreadBlock32x32       = uint2(GroupThread1D % BIN_TILE_SIZE, GroupThread1D / BIN_TILE_SIZE);
		const uint  Block8x8               = GroupThread1D / (RASTER_TILE_SIZE*RASTER_TILE_SIZE);

		const uint2 LocalThreadCoord_Bin   = ThreadBlock32x32;
		const uint2 GlobalTileCoord_Bin = uint2(GroupID % 16, GroupID / 16); // For debug
		
		const uint2 LocalTileCoord_Raster   = uint2(Block8x8 % 4u, Block8x8 / 4u); // Each bin tile is divided in to 16 (=4x4) raster tiles
		const uint2 LocalThreadCoord_Raster = uint2(ThreadBlock8x256.x, ThreadBlock8x256.y % RASTER_TILE_SIZE);

		// Bin coord
		Coord.Bin.Tile     = 0;
		Coord.Bin.Thread   = LocalThreadCoord_Bin;
		Coord.Bin.Tile1d   = 0;
		Coord.Bin.Thread1d = Coord.Bin.Thread.x + Coord.Bin.Thread.y * BIN_TILE_SIZE;

		// Raster coord
		Coord.Raster.Tile     = LocalTileCoord_Raster; // Local 4x4 tile coord
		Coord.Raster.Thread   = LocalThreadCoord_Raster;
		Coord.Raster.Tile1d   = Coord.Raster.Tile.x + Coord.Raster.Tile.y * RASTER_TILE_COUNT_2D;
		Coord.Raster.Thread1d = Coord.Raster.Thread.x + Coord.Raster.Thread.y * RASTER_TILE_SIZE;

		// Output coord Setup later for each work item
		Coord.Out.PixelCoord = 0;
	}

	const uint RasterizerIndex = GroupID;
	const uint WorkCount = min(RasterWorkAllocatedCount[0], MaxRasterWorkCount);
	const FMinMaxZ MinMaxZ = UnpackMinMaxZ(ViewMinMaxZ[0], ViewMinMaxZ[1]);

	group_Valid = true;
	GroupMemoryBarrierWithGroupSync();

	// Persistent thread loop for binning the clusters queue
	#if 0
	uint IterationIt = 0;
	while (IterationIt++ < MAX_THREAD_ITERATION_COUNT)
	#else
	for (uint IterationIt=0; IterationIt<MAX_THREAD_ITERATION_COUNT; ++IterationIt)
	#endif
	{
		// 1. Fetch work
		if (GroupThread1D == 0)
		{
			uint WorkFetchIndex = 0;
			InterlockedAdd(RasterWorkQueue[0], 1, WorkFetchIndex);
			group_WorkFetchIndex = WorkFetchIndex;
			if (WorkFetchIndex < WorkCount)
			{
				group_Work = UnpackRasterWork(RasterWork[WorkFetchIndex]);
			}
		}
		GroupMemoryBarrierWithGroupSync();

		const bool bValidWork = group_WorkFetchIndex < WorkCount;
		if (!bValidWork)
		{
			return;
		}

		// 2. Clear shared data
		{
			// Segment's bitmasks
			if (GroupThread1D < RASTER_TILE_COUNT_1D * MAX_SEGMENT_PER_RASTER_STEP)
			{
				const uint RasterTile1d = GroupThread1D / MAX_SEGMENT_PER_RASTER_STEP;
				const uint SliceIndex = GroupThread1D % MAX_SEGMENT_PER_RASTER_STEP;
				#if WAVE_RASTER == 0
				group_SegmentsBits[RasterTile1d][SliceIndex] = 0;
				#endif

				group_SceneMaxZ[GroupThread1D] = 0;
			}

			#if WAVE_RASTER == 1
			if (GroupThread1D < THREADGROUP_SIZE)
			{
				group_SegmentsBits[GroupThread1D] = 0;
				group_SegmentsColor[GroupThread1D] = 0;
			}
			#endif

			if (GroupThread1D < RASTER_TILE_COUNT_1D * 2)
			{
				group_CompletedWaves[GroupThread1D] = 0;
			}
		}
		float3 OutColor = 0;
		float OutTransmittance = 1.f;

		// 2.1 Update coord with the current work item data
		Coord.Bin.Tile       = group_Work.TileCoord;
		Coord.Bin.Tile1d     = 0; // Not used
		Coord.Out.PixelCoord = group_Work.TileCoord * BIN_TILE_SIZE + Coord.Raster.Tile * RASTER_TILE_SIZE + Coord.Raster.Thread;

		// 2.2 Compute scene max Z for each raster tile
		{
			const float SceneDepth = SceneDepthTexture.Load(uint3(Coord.Out.PixelCoord, 0));
			WaveInterlockedMax(group_SceneMaxZ[Coord.Raster.Tile1d], asuint(SceneDepth));
		}

		GroupMemoryBarrierWithGroupSync();

		// 3. Process work or exit if done
		if (bValidWork)
		{
			#if PERMUTATION_DEBUG
			FShaderPrintContext Ctx = InitShaderPrintContextAtCursorUnique(Coord.Out.PixelCoord, uint2(650, 200), uint2(350, 0));
			PlotWorkInfo(Ctx, Coord);
			PrintLineN(Ctx, group_WorkFetchIndex);
			PrintLineN(Ctx, IterationIt);

			Newline(Ctx);
			PrintLineN(Ctx, DispatchThreadID);
			PrintLineN(Ctx, GroupThread1D);
			PrintLineN(Ctx, GroupID);
			Newline(Ctx);
			#endif

			// 2.2 Load & raster segments in front-to-back order
			for (uint ZBinIt = 0; ZBinIt < group_Work.ZBinCount; ZBinIt++)
			{
				// All rasterizers within the tile are done
				#if COVERAGE_CULLING
				{
					const uint LaneIndex = WaveGetLaneIndex();
					const uint WavePerRasterTile = WaveGetLaneCount() == 32u ? 2u : 1u;
					const uint RasterCoverageIndex = LaneIndex % (RASTER_TILE_COUNT_1D * WavePerRasterTile);
					const bool bTileDone = WaveActiveAllTrue(group_CompletedWaves[RasterCoverageIndex] == 1);
					if (bTileDone)
					{
						break;
					}
				}
				#endif

				#if PERMUTATION_DEBUG
				Print(Ctx, TEXT("ZBinIt : "), FontOrange); Print(Ctx, ZBinIt, FontOrange); Newline(Ctx);
				PrintLineN(Ctx, group_WorkFetchIndex);
				PrintLineN(Ctx, IterationIt);
				Newline(Ctx);
				#endif

				const uint2 Data = ZBinData[group_Work.ZBinOffset + ZBinIt];
				const uint SegmentOffset = Data.x;
				const uint SegmentCount  = min(Data.y, MAX_SEGMENT_COUNT_PER_ZBIN);

				if (GroupThread1D < SegmentCount)
				{
					group_PackedSegments[GroupThread1D] = ZBinSegmentData[SegmentOffset + GroupThread1D];
				}

				// Sort segments
				{
					// Count segment per bin
					// Compute bin offset/size
					// InsertSegment
				}
				GroupMemoryBarrierWithGroupSync();

				#if PERMUTATION_DEBUG
				PrintLineN(Ctx, SegmentOffset);
				PrintLineN(Ctx, SegmentCount);
				const uint ActualNumSegmentToLoad = Data.y;
				PrintLineN(Ctx, ActualNumSegmentToLoad);
				#endif

				// Raster 8x8
				const bool bRasterize = true;
				if (bRasterize)
				{
					const float2 RasterTileAABBMin =  Coord.Raster.Tile    * RASTER_TILE_SIZE;
					const float2 RasterTileAABBMax = (Coord.Raster.Tile+1) * RASTER_TILE_SIZE;

					const uint WaveIndex = GroupThread1D / WaveGetLaneCount();
					const uint WaveOffset = WaveIndex * WaveGetLaneCount();
					const uint LaneIndex = WaveGetLaneIndex();
					const uint LaneCount = WaveGetLaneCount();

					#if PERMUTATION_DEBUG
					//Newline(Ctx);
					//PrintLineN(Ctx, WaveIndex);
					//PrintLineN(Ctx, WaveOffset);
					//PrintLineN(Ctx, LaneIndex);
					//PrintLineN(Ctx, LaneCount);
					#endif

					// ... 
					#if PERMUTATION_DEBUG
					const uint2 CursorPos = GetCursorPos();
					const float2 GlobalRasterTileAABBMin = Coord.Bin.Tile * BIN_TILE_SIZE +  Coord.Raster.Tile    * RASTER_TILE_SIZE;
					const float2 GlobalRasterTileAABBMax = Coord.Bin.Tile * BIN_TILE_SIZE + (Coord.Raster.Tile+1) * RASTER_TILE_SIZE;
					//PrintLineN(Ctx, GlobalRasterTileAABBMin);
					//PrintLineN(Ctx, GlobalRasterTileAABBMax);
					AddQuadSS(Ctx, GlobalRasterTileAABBMin, GlobalRasterTileAABBMax, ColorYellow);
					const bool bDebugRasterTile = all(CursorPos >= GlobalRasterTileAABBMin) && all(CursorPos < GlobalRasterTileAABBMax) && ZBinIt == 0 && IterationIt == 0;
					const uint2 LanePacked = (WaveIndex & 1) == 0 ? uint2(0,0) : uint2(0, 4);
					//FShaderPrintContext CtxU = InitShaderPrintContext(bDebugRasterTile, uint2(500 + LaneIndex * 20, 200 + Yoff * 10));
					const uint2 LaneXY = uint2(LaneIndex % 8, LaneIndex / 8);
					//FShaderPrintContext CtxU = InitShaderPrintContext(bDebugRasterTile, uint2(500, 200) + (LaneXY + LanePacked) * 20);
					//FShaderPrintContext CtxU = InitShaderPrintContext(bDebugRasterTile, uint2(500, 200) + (LaneXY + LanePacked) * uint2(120, 40));
					#endif

					const float Cov = 0.2f;
					for (uint SegOffset=0; SegOffset<SegmentCount; SegOffset+=MAX_SEGMENT_PER_RASTER_STEP) // sweep all segments, 32 at a time
					{
						// Compute segment's bitmask - in batch of MAX_SEGMENT_PER_RASTER_STEP(32)
						// Each wave compute the bitmask for 32 segments
						if (LaneIndex < MAX_SEGMENT_PER_RASTER_STEP)
						{
							uint2 SegmentBitmask = 0;
							float3 SegmentColor = 0;
							if (SegOffset + LaneIndex < SegmentCount)
							{
								// Segment is loaded relative to the current bin tile. We don't need the absolute coord.
								const FSegment Segment = UnpackSegment(0/*Coord.Bin.Tile*/, MinMaxZ, group_PackedSegments[SegOffset + LaneIndex]);
								SegmentColor = Segment.Color;

								float3 ClipP0 = Segment.P0;
								float3 ClipP1 = Segment.P1;
								if (ClipSegment(RasterTileAABBMin, RasterTileAABBMax, ClipP0, ClipP1))
								{
									const float2 nP0 = clamp(ClipP0.xy - RasterTileAABBMin, 0, RASTER_TILE_SIZE* 0.999f);
									const float2 nP1 = clamp(ClipP1.xy - RasterTileAABBMin, 0, RASTER_TILE_SIZE* 0.999f);

									const float PMinZ = min(ClipP0.z, ClipP1.z);
									const float PMaxZ = min(ClipP0.z, ClipP1.z);
									const bool bNeedFineDepthTest = PMinZ < asfloat(group_SceneMaxZ[Coord.Raster.Tile1d]);
									#if PERMUTATION_DEBUG
									if (0)//(bNeedFineDepthTest)
									{
										FShaderPrintContext CtxD = InitShaderPrintContext(true, 0);
										AddQuadSS(CtxD, GlobalRasterTileAABBMin, GlobalRasterTileAABBMax, ColorOrange);
									}
									#endif
									SegmentBitmask = GetSegmentBits(SceneDepthTexture, Coord.Bin.Tile * BIN_TILE_SIZE + RasterTileAABBMin, nP0, nP1, ClipP0.z, ClipP1.z, bNeedFineDepthTest);
								}
							}
							group_SegmentsBits[WaveOffset + LaneIndex] = SegmentBitmask;
							group_SegmentsColor[WaveOffset + LaneIndex] = SegmentColor.x;
						}

						// Compute raster tile bitmask (8x8)
						if (WaveGetLaneCount() == 32)
						{
							const uint ThreadMask = 1u << LaneIndex;

							// 0..31
							// x x x x x x x x
							// x x x x x x x x
							// x x x x x x x x
							// x x x x x x x x
							// . . . . . . . .
							// . . . . . . . .
							// . . . . . . . .
							// . . . . . . . .
							if ((WaveIndex&1) == 0)
							{
								for (uint SliceIt = 0; SliceIt < 32u; ++SliceIt)
								{
									const bool bVisible = (group_SegmentsBits[WaveOffset + SliceIt].x & ThreadMask) != 0;
									if (bVisible)
									{
										const float CurrentTransmittance = OutTransmittance;
										OutTransmittance *= (1-Cov);

										const float3 SegmentColor = group_SegmentsColor[WaveOffset + SliceIt];
										OutColor += OutTransmittance * Cov * SegmentColor.xyz;
									}
								}
							}
							// 32..63
							// . . . . . . . .
							// . . . . . . . .
							// . . . . . . . .
							// . . . . . . . .
							// x x x x x x x x
							// x x x x x x x x
							// x x x x x x x x
							// x x x x x x x x
							else
							{
								for (uint SliceIt = 0; SliceIt < 32u; ++SliceIt)
								{
									const bool bVisible = (group_SegmentsBits[WaveOffset + SliceIt].y & ThreadMask) != 0;
									if (bVisible)
									{
										const float CurrentTransmittance = OutTransmittance;
										OutTransmittance *= (1-Cov);

										const float3 SegmentColor = group_SegmentsColor[WaveOffset + SliceIt];
										OutColor += OutTransmittance * Cov * SegmentColor.xyz;
									}
								}
							}
						}
						else if (WaveGetLaneCount() == 64)
						{
							// TODO
						}

						#if COVERAGE_CULLING
						// Current rasterizer is done
						if (WaveActiveAllTrue(OutTransmittance < MinCoverageThreshold))
						{
							if (WaveIsFirstLane())
							{
								#if PERMUTATION_DEBUG
								FShaderPrintContext CtxD = InitShaderPrintContext(true, 0);
								const float3 TileColor = ColorMapTurbo(0.5f + (SegOffset / float(SegmentCount)) * 0.5f);
								//AddQuadSS(CtxD, GlobalRasterTileAABBMin, GlobalRasterTileAABBMax, float4(TileColor,1));
								PrintLineN(Ctx, OutTransmittance);
								PrintLine(Ctx, SegOffset); PrintLineN(Ctx, SegmentCount);
								#endif

								group_CompletedWaves[WaveIndex] = 1;
							}
							break;
						}
						#endif
					} // for (SegOffset)
					GroupMemoryBarrierWithGroupSync();
				} // if (bRasterize)
			} // for(ZBin)

			GroupMemoryBarrierWithGroupSync();

			#if PERMUTATION_DEBUG
			Newline(Ctx);
			PrintLineN(Ctx, OutColor);
			PrintLineN(Ctx, OutTransmittance);
			#endif

			// Write out
			//if (OutTransmittance < 1.f)
			{
				OutputTexture[Coord.Out.PixelCoord] = float4(OutColor, 1.f);
			}

		} // if (bValidWork)

	} // while()
}
#endif // SWRasterizerCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef DebugDrawingCS

#ifndef PERMUTATION_DEBUG
#error PERMUTATION_DEBUG needs to be defined 
#endif

uint TotalBufferMemoryInMBytes;
uint TotalTextureMemoryInMBytes;

Buffer<uint> VisibleInstanceArgs;
StructuredBuffer<uint2> VisibleInstances;

Buffer<uint> VisibleClusterArgs;
StructuredBuffer<uint2> VisibleClusters;

Texture2D<uint2> SceneTileDepthTexture;
StructuredBuffer<uint> ViewMinMaxZ;

#if PERMUTATION_SWRASTER
Texture2DArray<uint> TileSegmentCount;
StructuredBuffer<uint> TileDataAllocatedCount;

StructuredBuffer<uint2> ZBinData;
StructuredBuffer<uint2> RasterWork;
StructuredBuffer<uint> RasterWorkAllocatedCount;
StructuredBuffer<FPackedSegmentType> ZBinSegmentData;
StructuredBuffer<uint> ZBinSegmentAllocatedCount;
StructuredBuffer<uint> ZBinDataAllocatedCount;
#endif

void PrintRatio(inout FShaderPrintContext Ctx, uint In, uint InMax, uint InDigit)
{
	Print(Ctx, In, Select(In <= InMax, FontYellow, FontRed), InDigit,0); Print(Ctx, TEXT("/"), FontWhite); Print(Ctx, InMax, FontYellow, InDigit,0); 
}

[numthreads(THREADGROUP_SIZE, 1, 1)]
void DebugDrawingCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView(); 

	const uint VisibleInstanceCount = VisibleInstanceArgs[0];
	const uint VisibleClusterCount = VisibleClusterArgs[3];
	#if PERMUTATION_SWRASTER
	const uint TileDataAllocCount = TileDataAllocatedCount[0];
	const uint RasterWorkAllocCount = RasterWorkAllocatedCount[0];
	const uint ZBinSegAllocatedCount = ZBinSegmentAllocatedCount[0];
	const uint ZBinDatAllocatedCount = ZBinDataAllocatedCount[0];
	#endif

	const FMinMaxZ MinMaxZ = UnpackMinMaxZ(ViewMinMaxZ[0], ViewMinMaxZ[1]);

	// Draw main stats
	if (all(DispatchThreadId == 0))
	{
		// Draw main stats
		FShaderPrintContext Ctx = InitShaderPrintContext(all(DispatchThreadId == 0), uint2(50, 50));
		Print(Ctx, TEXT("Render Curve Raster Pipeline"), FontRed); Newline(Ctx);
		Newline(Ctx);

		Print(Ctx, TEXT("Instance/Cluster"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("Visible Instance    : "), FontWhite); PrintRatio(Ctx, VisibleInstanceCount, Scene.RenderCurve.InstanceCount, 3); Newline(Ctx);
		Print(Ctx, TEXT("Visible Cluster     : "), FontWhite); PrintRatio(Ctx, VisibleClusterCount, Scene.RenderCurve.ClusterCount, 6); Newline(Ctx);
		Print(Ctx, TEXT("Max ClusterStride   : "), FontWhite); Print(Ctx, Scene.RenderCurve.MaxClusterStrideInBytes, FontYellow);  Newline(Ctx);
		Newline(Ctx);

		Print(Ctx, TEXT("Min/Max Z"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("MinZ                : "), FontWhite); Print(Ctx, MinMaxZ.MinZ, FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("MaxZ                : "), FontWhite); Print(Ctx, MinMaxZ.MaxZ, FontYellow); Newline(Ctx);
		Newline(Ctx);

		#if PERMUTATION_SWRASTER
		Print(Ctx, TEXT("Segment & ZBin"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("MaxSegmentDataCount : "), FontWhite); Print(Ctx, MaxSegmentDataCount, FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("ZBin Data alloc.    : "), FontWhite); PrintRatio(Ctx, ZBinDatAllocatedCount, MaxZBinDataCount, 9); Newline(Ctx);
		Print(Ctx, TEXT("ZBin Segment alloc. : "), FontWhite); PrintRatio(Ctx, ZBinSegAllocatedCount, MaxZBinSegmentDataCount, 9); Newline(Ctx);
		Newline(Ctx);

		Print(Ctx, TEXT("Binners"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("Num Binners         : "), FontWhite); Print(Ctx, NumBinners, FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Bin Tile Size       : "), FontWhite); Print(Ctx, uint(BIN_TILE_SIZE), FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Bin Res             : "), FontWhite); Print(Ctx, BinTileRes.x, FontYellow, 3, 0);  Print(Ctx, TEXT("x"), FontWhite); Print(Ctx, BinTileRes.y, FontYellow, 3, 0);Newline(Ctx);
		Print(Ctx, TEXT("Tile data allocated : "), FontWhite); PrintRatio(Ctx, TileDataAllocCount, MaxTileDataCount, 8); Newline(Ctx);
		Newline(Ctx);

		Print(Ctx, TEXT("Rasterizers"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("Num Rasterizers     : "), FontWhite); Print(Ctx, NumRasterizers, FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Raster Tile Size    : "), FontWhite); Print(Ctx, uint(RASTER_TILE_SIZE), FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Raster Res          : "), FontWhite); Print(Ctx, RasterTileRes.x, FontYellow, 3, 0);  Print(Ctx, TEXT("x"), FontWhite); Print(Ctx, RasterTileRes.y, FontYellow, 3, 0);Newline(Ctx);
		Print(Ctx, TEXT("Raster Work         : "), FontWhite); PrintRatio(Ctx, RasterWorkAllocCount, MaxRasterWorkCount, 6); Newline(Ctx);
		Print(Ctx, TEXT("Raster Load         : "), FontWhite); Print(Ctx, RasterWorkAllocCount / float(NumRasterizers), FontYellow); Newline(Ctx);
		Newline(Ctx);
		#endif

		Print(Ctx, TEXT("Memory"), FontOrange); Newline(Ctx);
		Print(Ctx, TEXT("Buffer Memory(MB)   : "), FontWhite); Print(Ctx, TotalBufferMemoryInMBytes, FontYellow); Newline(Ctx);
		Print(Ctx, TEXT("Texture Memory(MB)  : "), FontWhite); Print(Ctx, TotalTextureMemoryInMBytes, FontYellow); Newline(Ctx);
		Newline(Ctx);

		// Cursor bin
		#if PERMUTATION_SWRASTER
		if (0)
		{
			const uint2 BinCoord = uint2(GetCursorPos()) >> BIN_TILE_SIZE_DIV_AS_SHIFT;
			uint SegmentCount = 0; 
			for (uint BinnerIt = 0; BinnerIt < NumBinners; ++BinnerIt)
			{
				SegmentCount += TileSegmentCount.Load(uint4(BinCoord, BinnerIt, 0));
			}
			AddQuadSS(Ctx, BinCoord * BIN_TILE_SIZE, (BinCoord + 1) * BIN_TILE_SIZE, ColorRed);
			Print(Ctx, TEXT("SegmentCount        : "), FontWhite); Print(Ctx, SegmentCount, FontYellow); Newline(Ctx);
		}
		#endif
	}

	// Draw bining tiles
	#if PERMUTATION_SWRASTER
	if (all(DispatchThreadId.xy < BinTileRes))
	{
		const uint2 BinCoord = DispatchThreadId.xy;
		// All bin
		if (0)
		{
			uint SegmentCount = 0; 
			for (uint BinnerIt = 0; BinnerIt < NumBinners; ++BinnerIt)
			{
				SegmentCount += TileSegmentCount.Load(uint4(BinCoord, BinnerIt, 0));
			}

			FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(50, 350));
			AddFilledQuadSS(Ctx, float2(BinCoord) * BinTileSize, float2(BinCoord + 1) * BinTileSize, float4(ColorMapMagma(SegmentCount/2048.f), 0.5));
			//AddQuadSS(Ctx, BinCoord * BIN_TILE_SIZE, (BinCoord + 1) * BIN_TILE_SIZE, ColorRed);
			//PrintLineN(Ctx, SegmentCount);
			
			//const uint2 BinCoord = uint2(BinX, BinY);
			//const float Depth = UnpackDepth(SceneTileDepthTexture.Load(uint3(BinCoord, 0))).x;
			//AddFilledQuadSS(Ctx, float2(BinCoord) * BinTileSize, float2(BinCoord + 1) * BinTileSize, float4(ColorMapMagma(Depth), 0.5));

			//const uint SegmentCount = TileSegmentCount.Load(uint4(BinCoord, ResolvedView.GeneralPurposeTweak, 0));
		}
	}
	#endif

	// Draw ZBin tiles
	#if PERMUTATION_SWRASTER
	if (0)
	//if (all(DispatchThreadId == 0))
	{
		FShaderPrintContext Ctx = InitShaderPrintContext(all(DispatchThreadId == 0), uint2(50, 350));

		const uint RasterWorkCount = RasterWorkAllocatedCount[0];
		const FMinMaxZ MinMaxZ = UnpackMinMaxZ(ViewMinMaxZ[0], ViewMinMaxZ[1]);
		for (uint WorkIt = 0; WorkIt < RasterWorkCount; ++WorkIt)
		{
			const FRasterWork Work = UnpackRasterWork(RasterWork[WorkIt]);
		
			#if PERMUTATION_DEBUG
			//PrintLine(Ctx, Work.TileCoord);
			//PrintLine(Ctx, Work.ZBinOffset);
			//PrintLine(Ctx, Work.ZBinCount);
			for (uint ZBinIt = 0; ZBinIt < Work.ZBinCount; ZBinIt++)
			{
				const uint2 Data = ZBinData[Work.ZBinOffset + ZBinIt];
				const uint SegmentOffset = Data.x;
				const uint SegmentCount  = Data.y;
				//PrintLine(Ctx, SegmentOffset);
				//PrintLine(Ctx, SegmentCount);

				for (uint SegIt = 0; SegIt < SegmentCount; SegIt++)
				//uint SegIt = 0;
				//if (SegmentCount>0)
				{
					const FPackedSegmentType PackedSegment = ZBinSegmentData[SegmentOffset + SegIt];
					const FSegment Segment = UnpackSegment(Work.TileCoord, MinMaxZ, PackedSegment);
					AddLineSS(Ctx, Segment.P0.xy, Segment.P1.xy, ColorPurple);

					//PrintLine(Ctx, Segment.P0.xy);
					//PrintLine(Ctx, Segment.P1.xy);
				}
			}
			Newline(Ctx);
			#endif
		}
	}
	#endif

	// Draw clusters
	#if 1
	const uint VisibleClusterFetchIndex = DispatchThreadId.x + DispatchThreadId.y * Resolution.x;
	if (VisibleClusterFetchIndex < VisibleClusterCount)
	{
		const uint2 VisibleData = VisibleClusters[VisibleClusterFetchIndex];
		const uint  PrimitiveIndex = VisibleData.x;
		const uint  ClusterIndex = VisibleData.y;

		const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);
		const FClusterHeader ClusterHeader = GetClusterHeader(ClusterIndex);

		const float3 LocalBoundsCenter = ClusterHeader.LocalBoundCenter;
		const float3 LocalBoundsExtent = ClusterHeader.LocalBoundExtent;

		// Cluster bounds
		#if 0
		{
			const float3 ClusterColor = ColorMapMagma(VisibleClusterFetchIndex / float(VisibleClusterCount));
			FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(0,0));
			AddOBBTWS(Ctx, LocalBoundsCenter - LocalBoundsExtent, LocalBoundsCenter + LocalBoundsExtent, float4(ClusterColor,1), RenderCurveInstanceData.LocalToTranslatedWorld);
		}
		#endif

		// Cluster curves
		#if 0
		for (uint CurveIt = 0; CurveIt < ClusterHeader.CurvePerCluster; ++CurveIt)
		{
			float3 PrevPoint = 0;
			for (uint PointIt = 0; PointIt < ClusterHeader.PointPerCurve; ++PointIt)
			{
				const FCurvePoint Point = GetClusterPoint(ClusterHeader, ClusterIndex, CurveIt, PointIt);
				const float3 TranslatedWorldPosition = mul(float4(Point.Position, 1), RenderCurveInstanceData.LocalToTranslatedWorld).xyz;
				if (PointIt > 0 && Point.bValid)
				{
					AddLineTWS(PrevPoint, TranslatedWorldPosition, lerp(ColorOrange, ColorBlue, float(PointIt) / ClusterHeader.PointPerCurve));
				}
				PrevPoint = TranslatedWorldPosition;
			}
		}
		#endif
	}
	#endif

	#if 0
	const uint2 VisibleData = VisibleClusters[GroupId];
	const uint  PrimitiveIndex = VisibleData.x;
	const uint  ClusterIndex = VisibleData.y;
	const uint  CurveIndex = LinearThreadIndex; // expect curve count == THREADGROUP_SIZE. Add validation code for this

	const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);
	const FClusterHeader ClusterHeader = GetClusterHeader(ClusterIndex);

	//float3 PrevPoint = 0;
	//for (uint32 PointIt = 0; PointIt < ClusterHeader.PointPerCurve; ++PointIt)
	//{
	//	const float3 Position = GetClusterPoint(ClusterHeader, CurveIndex, PointIt);
	//	const float3 TranslatedWorldPosition = mul(Position, RenderCurveInstanceData.LocalToTranslatedWorld);
	//	if (PointIt > 0)
	//	{
	//		AddLineTWS(PrevPoint, TranslatedWorldPosition, lerp(ColorOrange, ColorBlue, float(PointIt)/ClusterHeader.PointPerCurve));
	//	}
	//}
	#endif
}

#endif // DebugDrawingCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef SegmentLUTCS

#if PERMUTATION_DEBUG
void DrawBitLine(RWTexture2D<float4> Out, uint2 OutResolution, uint2 OutBaseCoord, uint2 In)
{	
	for (uint y=0;y<8; ++y)
	for (uint x=0;x<8; ++x)
	{
		const uint l = x + y * 8;
		const uint b = l<32u ? ((In.x>>l)&0x1) : ((In.y>>(l-32u))&0x1);
		if (b > 0)
		{
			const uint2 OutCoord = OutBaseCoord + uint2(x, y);
			if (all(OutCoord < OutResolution))
			Out[OutCoord] = float4(0,1,0,1);
		}
	}
}
#endif

uint2 DebugOutputResolution;
RWTexture2D<float4> RWDebugOutput;
RWTexture2D<uint2>  RWSegmentLUT;

#define LUT_RESOLUTION THREADGROUP_SIZE_X
#if THREADGROUP_SIZE_X != THREADGROUP_SIZE_Y
#define THREADGROUP_SIZE_X and THREADGROUP_SIZE_Y needs to have the same size
#endif

[numthreads(LUT_RESOLUTION, LUT_RESOLUTION, 1)]
void SegmentLUTCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint2 GroupThreadID2D : SV_GroupThreadID, uint2 GroupID : SV_GroupID)
{
	const float2 P0 = GroupID.xy + 0.5f;
	const float2 P1 = GroupThreadID2D.xy + 0.5f;

	// 1. Init 16x16 output
	uint Out[LUT_RESOLUTION][LUT_RESOLUTION];
	for (uint y=0;y<LUT_RESOLUTION; ++y)
	for (uint x=0;x<LUT_RESOLUTION; ++x)
	{
		Out[x][y] = 0;
	}

	// 2. Rasterize line (16x16)
	FDDAContext DDAContext = DDACreateContext(P0, normalize(P1 - P0));
	const int2 EndCoord = (int2)floor(P1.xy);
	for (int DDAIt = 0; DDAIt < DDA_MAX_ITERATIONS; ++DDAIt)
	{
		const int2 TileCoord = (int2)floor(DDAContext.Coord);

		if (all(TileCoord >= 0) && all(TileCoord < LUT_RESOLUTION))
		{
			Out[TileCoord.x][TileCoord.y] = 1;
		}

		if (all(TileCoord == EndCoord))
		{
			break;
		}

		DDAAdvance(DDAContext);
	}

	// 3. Downsample (16x16) -> (8x8)
	uint2 BitOutput = 0;
	for (uint y=0;y<LUT_RESOLUTION; y+=2)
	for (uint x=0;x<LUT_RESOLUTION; x+=2)
	{
		uint Avg = 
		Out[x  ][y  ] +
		Out[x+1][y  ] +
		Out[x+1][y+1] +
		Out[x  ][y+1] ;

		if (Avg >= 2)
		{
			const uint hx = x >> 1u;
			const uint hy = y >> 1u;
			const uint l = hx + hy * 8u;
			if (l < 32)
			{
				BitOutput.x |= 1u << l;
			}
			else
			{
				BitOutput.y |= 1u << (l-32u);
			}
		}
	}

	// 4. Write outptu
	RWSegmentLUT[DispatchThreadId.xy] = BitOutput;

	// 5. Plot output in 2D
	#if PERMUTATION_DEBUG
	if (0)
	{
		const uint2 TileSize = 16u;
		const uint2 BaseCoord = DispatchThreadId.xy * 16u;

		float4 DebugColor = float4(1, 0, 0, 1);
		const uint2 CursorCoord = uint2(ShaderPrintData.CursorCoord);
		FShaderPrintContext Ctx;
		if (all(BaseCoord <= CursorCoord) && all(CursorCoord <= BaseCoord + TileSize))
		{
			Ctx = InitShaderPrintContext(true, uint2(500, 50));
			Print(Ctx, TEXT("SegmentLUT"), FontRed); Newline(Ctx);
			PrintLineN(Ctx, P0);
			PrintLineN(Ctx, P1);

			AddQuadSS(Ctx, BaseCoord, BaseCoord + 16u, ColorYellow);
			DebugColor = float4(0, 1, 0, 1);
		}
		{

			DrawBitLine(RWDebugOutput, DebugOutputResolution, BaseCoord, BitOutput);
		}
	}
	#endif // PERMUTATION_DEBUG
}
#endif // SegmentLUTCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef IndirectArgsCS

Buffer<uint>				InVisibleClusterArgs;
RWBuffer<uint> 				OutHWRasterIndirectArgs;
RWStructuredBuffer<uint>	OutVisibleClusterCount;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void IndirectArgsCS(uint GroupThread1D : SV_GroupIndex, uint GroupID : SV_GroupID)
{
	if (GroupThread1D == 0)
	{
		const uint VisibleClusterCount = InVisibleClusterArgs[3];
		OutHWRasterIndirectArgs[0] = RENDERCURVE_CURVE_PER_CLUSTER;
		OutHWRasterIndirectArgs[1] = VisibleClusterCount;
		OutHWRasterIndirectArgs[2] = 1;

		OutVisibleClusterCount[0] = VisibleClusterCount;
	}
}
#endif // IndirectArgsCS

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if defined(HWRasterizerMS) || defined(HWRasterizerPS)
struct VSOut
{
	float4 Position	: SV_Position;
	float CoordU : TEXCOORD1;
};

struct VSPrimiveOut
{
	// Use uint4 to prevent compiler from erroneously packing per-vertex and per-prim attributes together
	nointerpolation uint PackedVisibleCurve : TEXCOORD2;
};

struct FVisibleCurve
{
	uint VisibleClusterIndex;	/* [0-65k] */
	uint CurveIndex; 			/* [0-64]  */
	uint PointIndex; 			/* [0-64]  */
};

uint PackCurveSegment(FVisibleCurve In)
{
	return (In.VisibleClusterIndex << 16u) | (In.CurveIndex << 8) | (In.PointIndex);
}

FVisibleCurve UnpackCurveSegment(uint In)
{
	FVisibleCurve Out;
	Out.VisibleClusterIndex = BitFieldExtractU32(In, 16, 16);
	Out.CurveIndex = BitFieldExtractU32(In, 8, 8);
	Out.PointIndex = BitFieldExtractU32(In, 8, 0);
	return Out;
}
#endif

#ifdef HWRasterizerMS

StructuredBuffer<uint>    ViewMinMaxZ;
Texture2D<uint2>          SceneTileDepthTexture;
StructuredBuffer<uint2>   VisibleClusters;
StructuredBuffer<uint>    VisibleClustersCount;

#ifndef MESHSHADER
#error This shader requires mesh shader support
#endif

// Dispatch layout
// ---------------
// 1 group  -> 1 curve
// 1 thread -> 1 point -> 2 vertices | 2 triangles
// 1 cluster == 64 group

MESH_SHADER_TRIANGLE_ATTRIBUTES(128)
void HWRasterizerMS(
	uint GroupThreadID : SV_GroupThreadID,
	uint3 GroupID : SV_GroupID,
	MESH_SHADER_VERTEX_EXPORT(VSOut, 128),
	MESH_SHADER_TRIANGLE_EXPORT(126),
	MESH_SHADER_PRIMITIVE_EXPORT(VSPrimiveOut, 126)
)
{
	ResolvedView = ResolveView();

	uint NumUniqueVerts = 0;
	uint3 VertIndexes = 0;
	
	bool bValid = false;
	float3 SP0 = 0;
	float3 SP1 = 0;
	
	// 1. Load data
	float4 ClipP0 = float4(INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT);
	float4 ClipP1 = ClipP0;
	float UCoord0 = 0;
	uint PointCount = 0;
	uint CurveCount = 0;
	float SignT = 1;
	const uint PointIt = GroupThreadID;
	const uint CurveIt = GroupID.x;
	const uint ClusterFetchIndex = GroupID.y; // TODO linearize .yz
	if (ClusterFetchIndex < VisibleClustersCount[0])
	{
		const uint2 VisibleData = VisibleClusters[ClusterFetchIndex];
		const uint PrimitiveIndex = VisibleData.x;
		const uint ClusterIndex = VisibleData.y;
		const FRenderCurveInstanceData RenderCurveInstanceData = GetRenderCurveInstanceData(PrimitiveIndex);
		const FClusterHeader ClusterHeader = GetClusterHeader(ClusterIndex);
		const float4x4 LocalToClip = mul(RenderCurveInstanceData.LocalToTranslatedWorld, ResolvedView.TranslatedWorldToClip);

		const uint PointIt0 = GroupThreadID;
		const uint PointIt1 = PointIt0 + 1 < ClusterHeader.PointPerCurve ? PointIt0 + 1 : PointIt0 - 1;
		SignT = PointIt0 + 1 < ClusterHeader.PointPerCurve ? 1 : -1;

		CurveCount = ClusterHeader.CurveCount;
		PointCount = ClusterHeader.CurveCount > 0 && CurveIt < ClusterHeader.CurveCount ? ClusterHeader.PointPerCurve : 0u;
		if (PointIt0 < PointCount && CurveIt < CurveCount)
		{
			const FCurvePoint Point0 = GetClusterPoint(ClusterHeader, ClusterIndex, CurveIt, PointIt0);
			const FCurvePoint Point1 = GetClusterPoint(ClusterHeader, ClusterIndex, CurveIt, PointIt1);
			if (Point0.bValid && Point1.bValid)
			{
				float4 ClipPosition0 = mul(float4(Point0.Position, 1), LocalToClip);
				float4 ClipPosition1 = mul(float4(Point1.Position, 1), LocalToClip);
				UCoord0 = Point0.UCoord;
				ClipP0 = ClipPosition0;
				ClipP1 = ClipPosition1;
				bValid = true;
			}
		}
	}

	// 2. Write out VertexCount/TriangleCount for the whole batch
	const uint VertexCount = 2 * PointCount;
	const uint TriangleCount = PointCount > 0 ? 2 * (PointCount - 1) : 0u;
	SetMeshOutputCounts(VertexCount, TriangleCount);

	// 3. Write Vertex / Triangle
	if (PointIt < PointCount && CurveIt < CurveCount)
	{
		const float2 CP0 = ClipP0.xy;
		const float2 CP1 = ClipP1.xy;
		const float2 CT  = normalize(CP1 - CP0) * SignT;
		const float2 CR  = abs(CT.x) > 0.001f ? float2(CT.y, -CT.x) : float2(-CT.y, CT.x);

		const float2 InvClipResolution = 2.f / Resolution;
		const float  ClipPixelSize = max(InvClipResolution.x, InvClipResolution.y);

		VSOut OutVL, OutVR;
		OutVL.Position = bValid ? float4(ClipP0.xy - CR * ClipPixelSize * ClipP0.w, ClipP0.zw) : ClipP0;
		OutVR.Position = bValid ? float4(ClipP0.xy + CR * ClipPixelSize * ClipP0.w, ClipP0.zw) : ClipP0;
		OutVL.CoordU = UCoord0;
		OutVR.CoordU = UCoord0;

		// Vertices
		{
			const uint VertexIndex = PointIt*2;
			MESH_SHADER_WRITE_VERTEX(VertexIndex + 0, OutVL);
			MESH_SHADER_WRITE_VERTEX(VertexIndex + 1, OutVR);
		}
		
		// Triangles
		const uint TriangleIndex = PointIt*2;
		if (TriangleIndex < TriangleCount)
		{
			// Triangle indices
			const uint3 OutT0 = uint3(0, 1, 2) + TriangleIndex;
			const uint3 OutT1 = uint3(1, 3, 2) + TriangleIndex;
			MESH_SHADER_WRITE_TRIANGLE(TriangleIndex + 0, OutT0);
			MESH_SHADER_WRITE_TRIANGLE(TriangleIndex + 1, OutT1);

			// Attributes
			FVisibleCurve VisibleCurve;
			VisibleCurve.VisibleClusterIndex = ClusterFetchIndex;
			VisibleCurve.CurveIndex = CurveIt;
			VisibleCurve.PointIndex = PointIt;
			VSPrimiveOut OutPrimAttributes;
			OutPrimAttributes.PackedVisibleCurve = PackCurveSegment(VisibleCurve);
			MESH_SHADER_WRITE_PRIMITIVE(TriangleIndex + 0, OutPrimAttributes);
			MESH_SHADER_WRITE_PRIMITIVE(TriangleIndex + 1, OutPrimAttributes);
		}
	}
}
#endif // HWRasterizerMS

#ifdef HWRasterizerPS

void HWRasterizerPS(
	in float4 SvPosition: SV_Position,
	in float InCoordU : TEXCOORD1,
	in uint InPackedAttribute : TEXCOORD2,
	out float4 OutColor0 : SV_Target0)
{
	const FVisibleCurve Curve = UnpackCurveSegment(InPackedAttribute);
	OutColor0 = float4(InCoordU, Curve.VisibleClusterIndex, Curve.CurveIndex, Curve.PointIndex);
}
#endif // HWRasterizerPS