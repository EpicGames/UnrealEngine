// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HaltonUtilities.ush: HLSL implementation for optimized evaluation of
	Halton sequences.
=============================================================================*/

// Copyright (c) 2012 Leonhard Gruenschloss (leonhard@gruenschloss.org)
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights to
// use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies
// of the Software, and to permit persons to whom the Software is furnished to do
// so, subject to the following conditions:
//
// The above copyright notice and this permission notice shall be included in
// all copies or substantial portions of the Software.
//
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
// SOFTWARE.

// The original code has been adapted to HLSL

#pragma once

// PVS-Studio lexer doesn't handle hex floating point constants properly and doesn't treat "-1" as part of the constant.
//-V:0x1.fffffcp-1:1064 

float Halton_Sample2(unsigned int Index)
{
	// int Dimension = 0;
	Index = (Index << 16) | (Index >> 16);
	Index = ((Index & 0x00ff00ff) << 8) | ((Index & 0xff00ff00) >> 8);
	Index = ((Index & 0x0f0f0f0f) << 4) | ((Index & 0xf0f0f0f0) >> 4);
	Index = ((Index & 0x33333333) << 2) | ((Index & 0xcccccccc) >> 2);
	Index = ((Index & 0x55555555) << 1) | ((Index & 0xaaaaaaaa) >> 1);
	Index = 0x3f800000u | (Index >> 9);
	return asfloat(Index) - 1.f;
}

float Halton_Sample3(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 1;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + (Index % 243u)] * 14348907u +
		Permutation[Offset + ((Index / 243u) % 243u)] * 59049u +
		Permutation[Offset + ((Index / 59049u) % 243u)] * 243u +
		Permutation[Offset + ((Index / 14348907u) % 243u)]) * float(0x1.fffffcp-1 / 3486784401u); // Results in [0,1).
}

float Halton_Sample5(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 2;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + (Index % 125u)] * 1953125u +
		Permutation[Offset + ((Index / 125u) % 125u)] * 15625u +
		Permutation[Offset + ((Index / 15625u) % 125u)] * 125u +
		Permutation[Offset + ((Index / 1953125u) % 125u)]) * float(0x1.fffffcp-1 / 244140625u); // Results in [0,1).
}

float Halton_Sample7(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 3;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 343u] * 117649u +
		Permutation[Offset + (Index / 343u) % 343u] * 343u +
		Permutation[Offset + (Index / 117649u) % 343u]) * float(0x1.fffffcp-1 / 40353607u); // Results in [0,1).
}

float Halton_Sample11(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 4;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 121u] * 1771561u +
		Permutation[Offset + (Index / 121u) % 121u] * 14641u +
		Permutation[Offset + (Index / 14641u) % 121u] * 121u +
		Permutation[Offset + (Index / 1771561u) % 121u]) * float(0x1.fffffcp-1 / 214358881u); // Results in [0,1).
}

float Halton_Sample13(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 5;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 169u] * 4826809u +
		Permutation[Offset + (Index / 169u) % 169u] * 28561u +
		Permutation[Offset + (Index / 28561u) % 169u] * 169u +
		Permutation[Offset + (Index / 4826809u) % 169u]) * float(0x1.fffffcp-1 / 815730721u); // Results in [0,1).
}

float Halton_Sample17(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 6;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 289u] * 83521u +
		Permutation[Offset + (Index / 289u) % 289u] * 289u +
		Permutation[Offset + (Index / 83521u) % 289u]) * float(0x1.fffffcp-1 / 24137569u); // Results in [0,1).
}

float Halton_Sample19(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 7;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 361u] * 130321u +
		Permutation[Offset + (Index / 361u) % 361u] * 361u +
		Permutation[Offset + (Index / 130321u) % 361u]) * float(0x1.fffffcp-1 / 47045881u); // Results in [0,1).
}

float Halton_Sample23(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 8;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 23u] * 148035889u +
		Permutation[Offset + (Index / 23u) % 23u] * 6436343u +
		Permutation[Offset + (Index / 529u) % 23u] * 279841u +
		Permutation[Offset + (Index / 12167u) % 23u] * 12167u +
		Permutation[Offset + (Index / 279841u) % 23u] * 529u +
		Permutation[Offset + (Index / 6436343u) % 23u] * 23u +
		Permutation[Offset + (Index / 148035889u) % 23u]) * float(0x1.fffffcp-1 / 3404825447u); // Results in [0,1).
}

float Halton_Sample29(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 9;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 29u] * 20511149u +
		Permutation[Offset + (Index / 29u) % 29u] * 707281u +
		Permutation[Offset + (Index / 841u) % 29u] * 24389u +
		Permutation[Offset + (Index / 24389u) % 29u] * 841u +
		Permutation[Offset + (Index / 707281u) % 29u] * 29u +
		Permutation[Offset + (Index / 20511149u) % 29u]) * float(0x1.fffffcp-1 / 594823321u); // Results in [0,1).
}

float Halton_Sample31(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 10;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 31u] * 28629151u +
		Permutation[Offset + (Index / 31u) % 31u] * 923521u +
		Permutation[Offset + (Index / 961u) % 31u] * 29791u +
		Permutation[Offset + (Index / 29791u) % 31u] * 961u +
		Permutation[Offset + (Index / 923521u) % 31u] * 31u +
		Permutation[Offset + (Index / 28629151u) % 31u]) * float(0x1.fffffcp-1 / 887503681u); // Results in [0,1).
}

float Halton_Sample37(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 11;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 37u] * 69343957u +
		Permutation[Offset + (Index / 37u) % 37u] * 1874161u +
		Permutation[Offset + (Index / 1369u) % 37u] * 50653u +
		Permutation[Offset + (Index / 50653u) % 37u] * 1369u +
		Permutation[Offset + (Index / 1874161u) % 37u] * 37u +
		Permutation[Offset + (Index / 69343957u) % 37u]) * float(0x1.fffffcp-1 / 2565726409u); // Results in [0,1).
}

float Halton_Sample41(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 12;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 41u] * 2825761u +
		Permutation[Offset + (Index / 41u) % 41u] * 68921u +
		Permutation[Offset + (Index / 1681u) % 41u] * 1681u +
		Permutation[Offset + (Index / 68921u) % 41u] * 41u +
		Permutation[Offset + (Index / 2825761u) % 41u]) * float(0x1.fffffcp-1 / 115856201u); // Results in [0,1).
}

float Halton_Sample43(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 13;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 43u] * 3418801u +
		Permutation[Offset + (Index / 43u) % 43u] * 79507u +
		Permutation[Offset + (Index / 1849u) % 43u] * 1849u +
		Permutation[Offset + (Index / 79507u) % 43u] * 43u +
		Permutation[Offset + (Index / 3418801u) % 43u]) * float(0x1.fffffcp-1 / 147008443u); // Results in [0,1).
}

float Halton_Sample47(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 14;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 47u] * 4879681u +
		Permutation[Offset + (Index / 47u) % 47u] * 103823u +
		Permutation[Offset + (Index / 2209u) % 47u] * 2209u +
		Permutation[Offset + (Index / 103823u) % 47u] * 47u +
		Permutation[Offset + (Index / 4879681u) % 47u]) * float(0x1.fffffcp-1 / 229345007u); // Results in [0,1).
}

float Halton_Sample53(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 15;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 53u] * 7890481u +
		Permutation[Offset + (Index / 53u) % 53u] * 148877u +
		Permutation[Offset + (Index / 2809u) % 53u] * 2809u +
		Permutation[Offset + (Index / 148877u) % 53u] * 53u +
		Permutation[Offset + (Index / 7890481u) % 53u]) * float(0x1.fffffcp-1 / 418195493u); // Results in [0,1).
}

float Halton_Sample59(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 16;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 59u] * 12117361u +
		Permutation[Offset + (Index / 59u) % 59u] * 205379u +
		Permutation[Offset + (Index / 3481u) % 59u] * 3481u +
		Permutation[Offset + (Index / 205379u) % 59u] * 59u +
		Permutation[Offset + (Index / 12117361u) % 59u]) * float(0x1.fffffcp-1 / 714924299u); // Results in [0,1).
}

float Halton_Sample61(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 17;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 61u] * 13845841u +
		Permutation[Offset + (Index / 61u) % 61u] * 226981u +
		Permutation[Offset + (Index / 3721u) % 61u] * 3721u +
		Permutation[Offset + (Index / 226981u) % 61u] * 61u +
		Permutation[Offset + (Index / 13845841u) % 61u]) * float(0x1.fffffcp-1 / 844596301u); // Results in [0,1).
}

float Halton_Sample67(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 18;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 67u] * 20151121u +
		Permutation[Offset + (Index / 67u) % 67u] * 300763u +
		Permutation[Offset + (Index / 4489u) % 67u] * 4489u +
		Permutation[Offset + (Index / 300763u) % 67u] * 67u +
		Permutation[Offset + (Index / 20151121u) % 67u]) * float(0x1.fffffcp-1 / 1350125107u); // Results in [0,1).
}

float Halton_Sample71(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 19;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 71u] * 25411681u +
		Permutation[Offset + (Index / 71u) % 71u] * 357911u +
		Permutation[Offset + (Index / 5041u) % 71u] * 5041u +
		Permutation[Offset + (Index / 357911u) % 71u] * 71u +
		Permutation[Offset + (Index / 25411681u) % 71u]) * float(0x1.fffffcp-1 / 1804229351u); // Results in [0,1).
}

float Halton_Sample73(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 20;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 73u] * 28398241u +
		Permutation[Offset + (Index / 73u) % 73u] * 389017u +
		Permutation[Offset + (Index / 5329u) % 73u] * 5329u +
		Permutation[Offset + (Index / 389017u) % 73u] * 73u +
		Permutation[Offset + (Index / 28398241u) % 73u]) * float(0x1.fffffcp-1 / 2073071593u); // Results in [0,1).
}

float Halton_Sample79(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 21;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 79u] * 38950081u +
		Permutation[Offset + (Index / 79u) % 79u] * 493039u +
		Permutation[Offset + (Index / 6241u) % 79u] * 6241u +
		Permutation[Offset + (Index / 493039u) % 79u] * 79u +
		Permutation[Offset + (Index / 38950081u) % 79u]) * float(0x1.fffffcp-1 / 3077056399u); // Results in [0,1).
}

float Halton_Sample83(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 22;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 83u] * 47458321u +
		Permutation[Offset + (Index / 83u) % 83u] * 571787u +
		Permutation[Offset + (Index / 6889u) % 83u] * 6889u +
		Permutation[Offset + (Index / 571787u) % 83u] * 83u +
		Permutation[Offset + (Index / 47458321u) % 83u]) * float(0x1.fffffcp-1 / 3939040643u); // Results in [0,1).
}

float Halton_Sample89(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 23;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 89u] * 704969u +
		Permutation[Offset + (Index / 89u) % 89u] * 7921u +
		Permutation[Offset + (Index / 7921u) % 89u] * 89u +
		Permutation[Offset + (Index / 704969u) % 89u]) * float(0x1.fffffcp-1 / 62742241u); // Results in [0,1).
}

float Halton_Sample97(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 24;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 97u] * 912673u +
		Permutation[Offset + (Index / 97u) % 97u] * 9409u +
		Permutation[Offset + (Index / 9409u) % 97u] * 97u +
		Permutation[Offset + (Index / 912673u) % 97u]) * float(0x1.fffffcp-1 / 88529281u); // Results in [0,1).
}

float Halton_Sample101(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 25;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 101u] * 1030301u +
		Permutation[Offset + (Index / 101u) % 101u] * 10201u +
		Permutation[Offset + (Index / 10201u) % 101u] * 101u +
		Permutation[Offset + (Index / 1030301u) % 101u]) * float(0x1.fffffcp-1 / 104060401u); // Results in [0,1).
}

float Halton_Sample103(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 26;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 103u] * 1092727u +
		Permutation[Offset + (Index / 103u) % 103u] * 10609u +
		Permutation[Offset + (Index / 10609u) % 103u] * 103u +
		Permutation[Offset + (Index / 1092727u) % 103u]) * float(0x1.fffffcp-1 / 112550881u); // Results in [0,1).
}

float Halton_Sample107(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 27;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 107u] * 1225043u +
		Permutation[Offset + (Index / 107u) % 107u] * 11449u +
		Permutation[Offset + (Index / 11449u) % 107u] * 107u +
		Permutation[Offset + (Index / 1225043u) % 107u]) * float(0x1.fffffcp-1 / 131079601u); // Results in [0,1).
}

float Halton_Sample109(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 28;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 109u] * 1295029u +
		Permutation[Offset + (Index / 109u) % 109u] * 11881u +
		Permutation[Offset + (Index / 11881u) % 109u] * 109u +
		Permutation[Offset + (Index / 1295029u) % 109u]) * float(0x1.fffffcp-1 / 141158161u); // Results in [0,1).
}

float Halton_Sample113(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 29;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 113u] * 1442897u +
		Permutation[Offset + (Index / 113u) % 113u] * 12769u +
		Permutation[Offset + (Index / 12769u) % 113u] * 113u +
		Permutation[Offset + (Index / 1442897u) % 113u]) * float(0x1.fffffcp-1 / 163047361u); // Results in [0,1).
}

float Halton_Sample127(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 30;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 127u] * 2048383u +
		Permutation[Offset + (Index / 127u) % 127u] * 16129u +
		Permutation[Offset + (Index / 16129u) % 127u] * 127u +
		Permutation[Offset + (Index / 2048383u) % 127u]) * float(0x1.fffffcp-1 / 260144641u); // Results in [0,1).
}

float Halton_Sample131(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 31;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 131u] * 2248091u +
		Permutation[Offset + (Index / 131u) % 131u] * 17161u +
		Permutation[Offset + (Index / 17161u) % 131u] * 131u +
		Permutation[Offset + (Index / 2248091u) % 131u]) * float(0x1.fffffcp-1 / 294499921u); // Results in [0,1).
}

float Halton_Sample137(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 32;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 137u] * 2571353u +
		Permutation[Offset + (Index / 137u) % 137u] * 18769u +
		Permutation[Offset + (Index / 18769u) % 137u] * 137u +
		Permutation[Offset + (Index / 2571353u) % 137u]) * float(0x1.fffffcp-1 / 352275361u); // Results in [0,1).
}

// TODO: Build out to 128 dimensions
float Halton_Sample139(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 33;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 139u] * 2685619u +
		Permutation[Offset + (Index / 139u) % 139u] * 19321u +
		Permutation[Offset + (Index / 19321u) % 139u] * 139u +
		Permutation[Offset + (Index / 2685619u) % 139u]) * float(0x1.fffffcp-1 / 373301041u); // Results in [0,1).
}

float Halton_Sample149(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 34;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 149u] * 3307949u +
		Permutation[Offset + (Index / 149u) % 149u] * 22201u +
		Permutation[Offset + (Index / 22201u) % 149u] * 149u +
		Permutation[Offset + (Index / 3307949u) % 149u]) * float(0x1.fffffcp-1 / 492884401u); // Results in [0,1).
}

float Halton_Sample151(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 35;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 151u] * 3442951u +
		Permutation[Offset + (Index / 151u) % 151u] * 22801u +
		Permutation[Offset + (Index / 22801u) % 151u] * 151u +
		Permutation[Offset + (Index / 3442951u) % 151u]) * float(0x1.fffffcp-1 / 519885601u); // Results in [0,1).
}

float Halton_Sample157(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 36;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 157u] * 3869893u +
		Permutation[Offset + (Index / 157u) % 157u] * 24649u +
		Permutation[Offset + (Index / 24649u) % 157u] * 157u +
		Permutation[Offset + (Index / 3869893u) % 157u]) * float(0x1.fffffcp-1 / 607573201u); // Results in [0,1).
}

float Halton_Sample163(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 37;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 163u] * 4330747u +
		Permutation[Offset + (Index / 163u) % 163u] * 26569u +
		Permutation[Offset + (Index / 26569u) % 163u] * 163u +
		Permutation[Offset + (Index / 4330747u) % 163u]) * float(0x1.fffffcp-1 / 705911761u); // Results in [0,1).
}

float Halton_Sample167(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 38;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 167u] * 4657463u +
		Permutation[Offset + (Index / 167u) % 167u] * 27889u +
		Permutation[Offset + (Index / 27889u) % 167u] * 167u +
		Permutation[Offset + (Index / 4657463u) % 167u]) * float(0x1.fffffcp-1 / 777796321u); // Results in [0,1).
}

float Halton_Sample173(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 39;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 173u] * 5177717u +
		Permutation[Offset + (Index / 173u) % 173u] * 29929u +
		Permutation[Offset + (Index / 29929u) % 173u] * 173u +
		Permutation[Offset + (Index / 5177717u) % 173u]) * float(0x1.fffffcp-1 / 895745041u); // Results in [0,1).
}

float Halton_Sample179(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 40;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 179u] * 5735339u +
		Permutation[Offset + (Index / 179u) % 179u] * 32041u +
		Permutation[Offset + (Index / 32041u) % 179u] * 179u +
		Permutation[Offset + (Index / 5735339u) % 179u]) * float(0x1.fffffcp-1 / 1026625681u); // Results in [0,1).
}

float Halton_Sample181(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 41;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 181u] * 5929741u +
		Permutation[Offset + (Index / 181u) % 181u] * 32761u +
		Permutation[Offset + (Index / 32761u) % 181u] * 181u +
		Permutation[Offset + (Index / 5929741u) % 181u]) * float(0x1.fffffcp-1 / 1073283121u); // Results in [0,1).
}

float Halton_Sample191(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 42;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 191u] * 6967871u +
		Permutation[Offset + (Index / 191u) % 191u] * 36481u +
		Permutation[Offset + (Index / 36481u) % 191u] * 191u +
		Permutation[Offset + (Index / 6967871u) % 191u]) * float(0x1.fffffcp-1 / 1330863361u); // Results in [0,1).
}

float Halton_Sample193(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 43;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 193u] * 7189057u +
		Permutation[Offset + (Index / 193u) % 193u] * 37249u +
		Permutation[Offset + (Index / 37249u) % 193u] * 193u +
		Permutation[Offset + (Index / 7189057u) % 193u]) * float(0x1.fffffcp-1 / 1387488001u); // Results in [0,1).
}

float Halton_Sample197(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 44;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 197u] * 7645373u +
		Permutation[Offset + (Index / 197u) % 197u] * 38809u +
		Permutation[Offset + (Index / 38809u) % 197u] * 197u +
		Permutation[Offset + (Index / 7645373u) % 197u]) * float(0x1.fffffcp-1 / 1506138481u); // Results in [0,1).
}

float Halton_Sample199(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 45;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 199u] * 7880599u +
		Permutation[Offset + (Index / 199u) % 199u] * 39601u +
		Permutation[Offset + (Index / 39601u) % 199u] * 199u +
		Permutation[Offset + (Index / 7880599u) % 199u]) * float(0x1.fffffcp-1 / 1568239201u); // Results in [0,1).
}

float Halton_Sample211(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 46;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 211u] * 9393931u +
		Permutation[Offset + (Index / 211u) % 211u] * 44521u +
		Permutation[Offset + (Index / 44521u) % 211u] * 211u +
		Permutation[Offset + (Index / 9393931u) % 211u]) * float(0x1.fffffcp-1 / 1982119441u); // Results in [0,1).
}

float Halton_Sample223(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 47;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 223u] * 11089567u +
		Permutation[Offset + (Index / 223u) % 223u] * 49729u +
		Permutation[Offset + (Index / 49729u) % 223u] * 223u +
		Permutation[Offset + (Index / 11089567u) % 223u]) * float(0x1.fffffcp-1 / 2472973441u); // Results in [0,1).
}

float Halton_Sample227(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 48;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 227u] * 11697083u +
		Permutation[Offset + (Index / 227u) % 227u] * 51529u +
		Permutation[Offset + (Index / 51529u) % 227u] * 227u +
		Permutation[Offset + (Index / 11697083u) % 227u]) * float(0x1.fffffcp-1 / 2655237841u); // Results in [0,1).
}

float Halton_Sample229(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 49;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 229u] * 12008989u +
		Permutation[Offset + (Index / 229u) % 229u] * 52441u +
		Permutation[Offset + (Index / 52441u) % 229u] * 229u +
		Permutation[Offset + (Index / 12008989u) % 229u]) * float(0x1.fffffcp-1 / 2750058481u); // Results in [0,1).
}

float Halton_Sample233(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 50;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 233u] * 12649337u +
		Permutation[Offset + (Index / 233u) % 233u] * 54289u +
		Permutation[Offset + (Index / 54289u) % 233u] * 233u +
		Permutation[Offset + (Index / 12649337u) % 233u]) * float(0x1.fffffcp-1 / 2947295521u); // Results in [0,1).
}

float Halton_Sample239(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 51;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 239u] * 13651919u +
		Permutation[Offset + (Index / 239u) % 239u] * 57121u +
		Permutation[Offset + (Index / 57121u) % 239u] * 239u +
		Permutation[Offset + (Index / 13651919u) % 239u]) * float(0x1.fffffcp-1 / 3262808641u); // Results in [0,1).
}

float Halton_Sample241(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 52;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 241u] * 13997521u +
		Permutation[Offset + (Index / 241u) % 241u] * 58081u +
		Permutation[Offset + (Index / 58081u) % 241u] * 241u +
		Permutation[Offset + (Index / 13997521u) % 241u]) * float(0x1.fffffcp-1 / 3373402561u); // Results in [0,1).
}

float Halton_Sample251(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 53;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 251u] * 15813251u +
		Permutation[Offset + (Index / 251u) % 251u] * 63001u +
		Permutation[Offset + (Index / 63001u) % 251u] * 251u +
		Permutation[Offset + (Index / 15813251u) % 251u]) * float(0x1.fffffcp-1 / 3969126001u); // Results in [0,1).
}

float Halton_Sample257(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 54;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 257u] * 66049u +
		Permutation[Offset + (Index / 257u) % 257u] * 257u +
		Permutation[Offset + (Index / 66049u) % 257u]) * float(0x1.fffffcp-1 / 16974593u); // Results in [0,1).
}

float Halton_Sample263(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 55;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 263u] * 69169u +
		Permutation[Offset + (Index / 263u) % 263u] * 263u +
		Permutation[Offset + (Index / 69169u) % 263u]) * float(0x1.fffffcp-1 / 18191447u); // Results in [0,1).
}

float Halton_Sample269(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 56;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 269u] * 72361u +
		Permutation[Offset + (Index / 269u) % 269u] * 269u +
		Permutation[Offset + (Index / 72361u) % 269u]) * float(0x1.fffffcp-1 / 19465109u); // Results in [0,1).
}

float Halton_Sample271(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 57;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 271u] * 73441u +
		Permutation[Offset + (Index / 271u) % 271u] * 271u +
		Permutation[Offset + (Index / 73441u) % 271u]) * float(0x1.fffffcp-1 / 19902511u); // Results in [0,1).
}

float Halton_Sample277(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 58;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 277u] * 76729u +
		Permutation[Offset + (Index / 277u) % 277u] * 277u +
		Permutation[Offset + (Index / 76729u) % 277u]) * float(0x1.fffffcp-1 / 21253933u); // Results in [0,1).
}

float Halton_Sample281(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 59;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 281u] * 78961u +
		Permutation[Offset + (Index / 281u) % 281u] * 281u +
		Permutation[Offset + (Index / 78961u) % 281u]) * float(0x1.fffffcp-1 / 22188041u); // Results in [0,1).
}

float Halton_Sample283(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 60;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 283u] * 80089u +
		Permutation[Offset + (Index / 283u) % 283u] * 283u +
		Permutation[Offset + (Index / 80089u) % 283u]) * float(0x1.fffffcp-1 / 22665187u); // Results in [0,1).
}

float Halton_Sample293(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 61;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 293u] * 85849u +
		Permutation[Offset + (Index / 293u) % 293u] * 293u +
		Permutation[Offset + (Index / 85849u) % 293u]) * float(0x1.fffffcp-1 / 25153757u); // Results in [0,1).
}

float Halton_Sample307(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 62;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 307u] * 94249u +
		Permutation[Offset + (Index / 307u) % 307u] * 307u +
		Permutation[Offset + (Index / 94249u) % 307u]) * float(0x1.fffffcp-1 / 28934443u); // Results in [0,1).
}

float Halton_Sample311(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 63;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 311u] * 96721u +
		Permutation[Offset + (Index / 311u) % 311u] * 311u +
		Permutation[Offset + (Index / 96721u) % 311u]) * float(0x1.fffffcp-1 / 30080231u); // Results in [0,1).
}

float Halton_Sample313(unsigned int Index, PERMUTATION_BUFFER<int> Permutation, PERMUTATION_BUFFER<int> PermutationOffset)
{
	int Dimension = 64;
	int Offset = PermutationOffset[Dimension - 1];
	return (Permutation[Offset + Index % 313u] * 97969u +
		Permutation[Offset + (Index / 313u) % 313u] * 313u +
		Permutation[Offset + (Index / 97969u) % 313u]) * float(0x1.fffffcp-1 / 30664297u); // Results in [0,1).
}
