// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

uint MurmurAdd(uint Hash, uint Element)
{
	Element *= 0xcc9e2d51;
	Element = (Element << 15) | (Element >> (32 - 15));
	Element *= 0x1b873593;

	Hash ^= Element;
	Hash = (Hash << 13) | (Hash >> (32 - 13));
	Hash = Hash * 5 + 0xe6546b64;
	return Hash;
}

uint MurmurMix(uint Hash)
{
	Hash ^= Hash >> 16;
	Hash *= 0x85ebca6b;
	Hash ^= Hash >> 13;
	Hash *= 0xc2b2ae35;
	Hash ^= Hash >> 16;
	return Hash;
}

// [Jarzynski 2020, "Hash Functions for GPU Rendering"]
uint PCGHash(uint Input)
{
    uint State = Input * 747796405u + 2891336453u;
    uint Word = ((State >> ((State >> 28u) + 4u)) ^ State) * 277803737u;
    return (Word >> 22u) ^ Word;
}

// A bit slower but higher quality than PCGHash
// https://github.com/Cyan4973/xxHash
uint XXHash32(uint Input)
{
	const uint PRIME32_2 = 2246822519U;
	const uint PRIME32_3 = 3266489917U;
	const uint PRIME32_4 = 668265263U;
	const uint PRIME32_5 = 374761393U;

	uint Hash = Input + PRIME32_5;
	Hash = PRIME32_4 * ((Hash << 17) | (Hash >> (32 - 17)));
    Hash = PRIME32_2 * (Hash ^ (Hash >> 15));
    Hash = PRIME32_3 * (Hash ^ (Hash >> 13));
    return Hash ^ (Hash >> 16);
}