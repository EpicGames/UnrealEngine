// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Hash.ush"

// Linear probing hash table
StructuredBuffer< uint >	HashTable;
RWStructuredBuffer< uint >	RWHashTable;
uint						HashTableSize;

/*
	Suggested use:

	If the original Key fits in a uint then pass it in as Key and hash it for the initial Index.
	If the original Key doesn't fit in a uint then use 2 different hashing functions for Key and initial Index.

	In either case a match must match both which reduces aliasing from hash collisions.
	This is a linear probing hash table so buckets aren't completely disjoint but this tends to
	double the effective Key bits in terms of likelihood for aliasing.
*/

// Returns true if key is added for the first time.
// Index input is the first hash table bucket to look in.
// Index output is the hash table bucket this key is stored in.
bool HashTableAdd( uint Key, inout uint Index )
{
	// Zero is reserved as invalid
	Key++;

	// TODO v_mul_hi_u32 to avoid integer division
	// ( Index * HashTableSize ) >> 32

	Index = Index % HashTableSize;

	LOOP
	ALLOW_UAV_CONDITION
	for( uint i = 0; i < HashTableSize; i++ )
	{
		uint StoredKey = RWHashTable[ Index ];
		if( StoredKey != Key )
		{
			if( StoredKey != 0 )
			{
				Index++;
				Index = Index >= HashTableSize ? 0 : Index;
				continue;
			}

			uint PrevKey;
			InterlockedCompareExchange( RWHashTable[ Index ], 0, Key, PrevKey );
			
			if( PrevKey == 0 )
				return true;
			else if( PrevKey != Key )
			{
				Index++;
				Index = Index >= HashTableSize ? 0 : Index;
				continue;
			}
		}

		break;
	}

	return false;
}

// Returns true if key is found.
// Index input is the first hash table bucket to look in.
// Index output is the hash table bucket this key is stored in if found.
bool HashTableFind( uint Key, inout uint Index )
{
	// Zero is reserved as invalid
	Key++;

	Index = Index % HashTableSize;

	LOOP
	ALLOW_UAV_CONDITION
	for( uint i = 0; i < HashTableSize; i++ )
	{
		uint StoredKey = HashTable[ Index ];
		if( StoredKey != Key )
		{
			if( StoredKey != 0 )
			{
				Index++;
				Index = Index >= HashTableSize ? 0 : Index;
				continue;
			}

			return false;
		}

		break;
	}

	return true;
}
