// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HdrCustomResolve.usf: Custom resolve for HDR color and box filter.
=============================================================================*/

#include "Common.ush"

#ifndef HDR_CUSTOM_RESOLVE_TEXTUREARRAY
	#define HDR_CUSTOM_RESOLVE_TEXTUREARRAY 0
#endif 

#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
	#define		MSTextureType(Type, NumSamples)		Texture2DMSArray<Type, NumSamples>
	#define		MSCoords							uint3
	#define		MSCoordsFromFloat4(Pos)				uint3(Pos.x, Pos.y, ViewID)
#else
	#define		MSTextureType(Type, NumSamples)		Texture2DMS<Type, NumSamples>
	#define		MSCoords							uint2
	#define		MSCoordsFromFloat4(Pos)				uint2(Pos.xy);
#endif

#ifdef OVERRIDE_HDRCUSTOMRESOLVESHADERS_USH
# include "/Platform/Private/HdrCustomResolveShaders.ush"
#else

void HdrCustomResolveVS(in uint InstanceID : SV_InstanceID, in uint Id : SV_VertexID,
	out float4 OutPosition				: SV_POSITION
#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
	,out nointerpolation uint OutViewId : VIEW_ID,
	out uint OutRTIndex					: SV_RenderTargetArrayIndex
#endif
	)
{
	int x = Id &  1;
	int y = Id >> 1;

	OutPosition = float4(x * 4 - 1, y * 4 - 1, 0, 1);

#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
	OutViewId = (InstanceID & 1);
	OutRTIndex = OutViewId;
#endif
}

float4 Encode(float4 Color)
{
	return float4(Color.rgb * rcp(Color.r*0.299 + Color.g*0.587 + Color.b*0.114 + 1.0), Color.a);
}

float4 Decode(float4 Color)
{
	return float4(Color.rgb * rcp(Color.r*(-0.299) + Color.g*(-0.587) + Color.b*(-0.114) + 1.0), Color.a);
}

#if HDR_CUSTOM_RESOLVE_USES_FMASK
// FMASK not implemented for this platform, so we do not actually use it here.
Texture2D<int2> FMaskTex;

int2 fmask;

# if HDR_RESOLVE_NUM_SAMPLES
void LoadTexSample(MSTextureType(float4, HDR_RESOLVE_NUM_SAMPLES) TargetTex, int2 P, int sampleIndex, inout float4 sampleSum, inout float weightSum)
{
	sampleSum += Encode(TargetTex.Load(P, sampleIndex));
	weightSum += 1;
}

MSTextureType(float4, HDR_RESOLVE_NUM_SAMPLES) Tex;
float4 HdrCustomResolveFMaskPS(float4 Pos : SV_POSITION
							#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
								,nointerpolation uint ViewID  : VIEW_ID
							#endif
	) : SV_Target0
{
	MSCoords P = MSCoordsFromFloat4(Pos);

	float4 sampleSum = float4(0, 0, 0, 0);
	float weightSum = 0;

	for (int i = 0; i < HDR_RESOLVE_NUM_SAMPLES; ++i)
	{
		LoadTexSample(Tex, P, i, sampleSum, weightSum);
	}
	return Decode(sampleSum / weightSum);
}
# endif
#else

# if HDR_CUSTOM_RESOLVE_2X

	MSTextureType(float4,2) Tex;
	float4 HdrCustomResolve2xPS(float4 Pos : SV_POSITION
							#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
								,nointerpolation uint ViewID  : VIEW_ID
							#endif
		) : SV_Target0
	{
		MSCoords P = MSCoordsFromFloat4(Pos);
		float4 C0 = Encode(Tex.Load(P, 0));
		float4 C1 = Encode(Tex.Load(P, 1));
		return Decode(C0*0.5 + C1*0.5);
	}
	
# endif

# if HDR_CUSTOM_RESOLVE_4X

	MSTextureType(float4,4) Tex;
	float4 HdrCustomResolve4xPS(float4 Pos : SV_POSITION
							#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
								,nointerpolation uint ViewID  : VIEW_ID
							#endif
		) : SV_Target0
	{
		MSCoords P = MSCoordsFromFloat4(Pos);
		float4 C0 = Encode(Tex.Load(P, 0));
		float4 C1 = Encode(Tex.Load(P, 1));
		float4 C2 = Encode(Tex.Load(P, 2));
		float4 C3 = Encode(Tex.Load(P, 3));
		return Decode(C0*0.25 + C1*0.25 + C2*0.25 + C3*0.25);
	}

# endif

# if HDR_CUSTOM_RESOLVE_8X

	MSTextureType(float4,8) Tex;
	float4 HdrCustomResolve8xPS(float4 Pos : SV_POSITION
							#if HDR_CUSTOM_RESOLVE_TEXTUREARRAY
								,nointerpolation uint ViewID  : VIEW_ID
							#endif
		) : SV_Target0
	{
		MSCoords P = MSCoordsFromFloat4(Pos);
		float4 C0 = Encode(Tex.Load(P, 0));
		float4 C1 = Encode(Tex.Load(P, 1));
		float4 C2 = Encode(Tex.Load(P, 2));
		float4 C3 = Encode(Tex.Load(P, 3));
		float4 C4 = Encode(Tex.Load(P, 4));
		float4 C5 = Encode(Tex.Load(P, 5));
		float4 C6 = Encode(Tex.Load(P, 6));
		float4 C7 = Encode(Tex.Load(P, 7));
		return Decode(C0*0.125 + C1*0.125 + C2*0.125 + C3*0.125 + C4*0.125 + C5*0.125 + C6*0.125 + C7*0.125);
	}
# endif

#endif

#endif
