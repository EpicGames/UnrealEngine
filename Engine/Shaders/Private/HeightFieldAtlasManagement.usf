// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightFieldAtlasManagement.usf
=============================================================================*/

#include "Common.ush"

uint4 UpdateRegionOffsetAndSize;
float4 SourceScaleBias;
uint SourceMipBias;

Texture2D<float4> SourceTexture;
SamplerState SourceTextureSampler;

RWTexture2D<float2> RWHeightFieldAtlas;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void UploadHeightFieldToAtlasCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId.xy < UpdateRegionOffsetAndSize.zw))
	{
		float2 SourceUV = DispatchThreadId.xy * SourceScaleBias.xy + SourceScaleBias.zw;
		uint2 WriteCoord = DispatchThreadId.xy + UpdateRegionOffsetAndSize.xy;

		RWHeightFieldAtlas[WriteCoord] = Texture2DSampleLevel(SourceTexture, SourceTextureSampler, SourceUV, SourceMipBias).xy;
	}
}

float4 VisibilityChannelMask;
RWTexture2D<float> RWVisibilityAtlas;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void UploadVisibilityToAtlasCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId.xy < UpdateRegionOffsetAndSize.zw))
	{
		float2 SourceUV = DispatchThreadId.xy * SourceScaleBias.xy + SourceScaleBias.zw;
		uint2 WriteCoord = DispatchThreadId.xy + UpdateRegionOffsetAndSize.xy;

		RWVisibilityAtlas[WriteCoord] = dot(Texture2DSampleLevel(SourceTexture, SourceTextureSampler, SourceUV, SourceMipBias), VisibilityChannelMask);
	}
}
