// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightFogCommon.usf:  
=============================================================================*/

#pragma once

#include "Common.ush"
#include "/Engine/Shared/EnvironmentComponentsFlags.h"
#include "/Engine/Private/SkyAtmosphereCommon.ush"
#include "ParticipatingMediaCommon.ush"
#include "ReflectionEnvironmentShared.ush"

#ifndef PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE
#define PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE 1
#endif

#ifndef PERMUTATION_SUPPORT_FOG_DIRECTIONAL_LIGHT_INSCATTERING
#define PERMUTATION_SUPPORT_FOG_DIRECTIONAL_LIGHT_INSCATTERING 1
#endif

#ifndef PERMUTATION_SUPPORT_VOLUMETRIC_FOG
#define PERMUTATION_SUPPORT_VOLUMETRIC_FOG 1
#endif

#ifndef PERMUTATION_SUPPORT_FOG_START_DISTANCE
#define PERMUTATION_SUPPORT_FOG_START_DISTANCE 1
#endif

#ifndef PERMUTATION_SUPPORT_FOG_SECOND_TERM
#define PERMUTATION_SUPPORT_FOG_SECOND_TERM 1
#endif

#ifndef FOG_MATERIALBLENDING_OVERRIDE
#define FOG_MATERIALBLENDING_OVERRIDE 0
#endif

#ifndef NEEDS_BASEPASS_VERTEX_FOGGING
#define NEEDS_BASEPASS_VERTEX_FOGGING 0
#endif

#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER
	#define FogStructISR MobileBasePass.Fog
#elif INSTANCED_STEREO
	#if MATERIALBLENDING_ANY_TRANSLUCENT
		#define FogStructISR TranslucentBasePass.Shared.FogISR
	#else
		#define FogStructISR OpaqueBasePass.Shared.FogISR
	#endif
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedIntegratedLightScatteringSampler	View.SharedBilinearClampedSampler
	#define SharedFogInColorScatteringSampler		View.SharedTrilinearClampedSampler
#else
	#define SharedIntegratedLightScatteringSampler	FogStruct.IntegratedLightScatteringSampler
	#define SharedFogInColorScatteringSampler		FogStruct.FogInscatteringColorSampler
#endif

static const float FLT_EPSILON = 0.001f;
static const float FLT_EPSILON2 = 0.01f;

// FogStruct.ExponentialFogParameters: FogDensity * exp2(-FogHeightFalloff * (CameraWorldPosition.z - FogHeight)) in x, FogHeightFalloff in y, MaxWorldObserverHeight in z, StartDistance in w. 
// FogStruct.ExponentialFogParameters2: FogDensitySecond * exp2(-FogHeightFalloffSecond * (CameraWorldPosition.z - FogHeightSecond)) in x, FogHeightFalloffSecond in y, FogDensitySecond in z, FogHeightSecond in w 
// FogStruct.ExponentialFogParameters3: FogDensity in x, FogHeight in y, whether to use cubemap fog color in z, FogCutoffDistance in w. 
// FogStruct.FogInscatteringTextureParameters: mip distance scale in x, bias in y, num mips in z 

struct FogStructData
{
	float4 ExponentialFogParameters;
	float4 ExponentialFogParameters2;
	float4 ExponentialFogColorParameter;
	float4 ExponentialFogParameters3;
	float4 SkyAtmosphereAmbientContributionColorScale;
	float4 InscatteringLightDirection;
	float4 DirectionalInscatteringColor;
	float2 SinCosInscatteringColorCubemapRotation;
	float3 FogInscatteringTextureParameters;
	float EndDistance;
	float ApplyVolumetricFog;
	float VolumetricFogStartDistance;
	float VolumetricFogNearFadeInDistanceInv;
	float3 VolumetricFogAlbedo;
	float VolumetricFogPhaseG;
	float SkyLightCaptureAffectsHeightFogStrength;
	float SkyLightCaptureAffectsHeightFogRoughness;
	float SkyLightCaptureAffectsHeightFogMipCount;
	// Spir-v does not support copying texture types so 
	// we can't use FogStructData to store them.
	//TextureCube FogInscatteringColorCubemap;
	//Texture3D IntegratedLightScattering;
};

FogStructData GetFogStructData(uint EyeIndex)
{
	FogStructData Out;
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER
	BRANCH
	if (EyeIndex == 0)
	{
#endif
		Out.ExponentialFogParameters = FogStruct.ExponentialFogParameters;
		Out.ExponentialFogParameters2 = FogStruct.ExponentialFogParameters2;
		Out.ExponentialFogColorParameter = FogStruct.ExponentialFogColorParameter;
		Out.ExponentialFogParameters3 = FogStruct.ExponentialFogParameters3;
		Out.SkyAtmosphereAmbientContributionColorScale = FogStruct.SkyAtmosphereAmbientContributionColorScale;
		Out.InscatteringLightDirection = FogStruct.InscatteringLightDirection;
		Out.DirectionalInscatteringColor = FogStruct.DirectionalInscatteringColor;
		Out.SinCosInscatteringColorCubemapRotation = FogStruct.SinCosInscatteringColorCubemapRotation;
		Out.FogInscatteringTextureParameters = FogStruct.FogInscatteringTextureParameters;
		Out.EndDistance = FogStruct.EndDistance;
		Out.ApplyVolumetricFog = FogStruct.ApplyVolumetricFog;
		Out.VolumetricFogStartDistance = FogStruct.VolumetricFogStartDistance;
		Out.VolumetricFogNearFadeInDistanceInv = FogStruct.VolumetricFogNearFadeInDistanceInv;
		Out.VolumetricFogAlbedo = FogStruct.VolumetricFogAlbedo;
		Out.VolumetricFogPhaseG = FogStruct.VolumetricFogPhaseG;
		Out.SkyLightCaptureAffectsHeightFogStrength  = FogStruct.SkyLightCaptureAffectsHeightFogStrength;
		Out.SkyLightCaptureAffectsHeightFogRoughness = FogStruct.SkyLightCaptureAffectsHeightFogRoughness;
		Out.SkyLightCaptureAffectsHeightFogMipCount  = FogStruct.SkyLightCaptureAffectsHeightFogMipCount;
		// Intentionally commented out to avoid spir-v generation issue - copying texture types is not supported
		//Out.FogInscatteringColorCubemap = FogStruct.FogInscatteringColorCubemap;
		//Out.IntegratedLightScattering = FogStruct.IntegratedLightScattering;
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER
	}
	else
	{	
		Out.ExponentialFogParameters = FogStructISR.ExponentialFogParameters;
		Out.ExponentialFogParameters2 = FogStructISR.ExponentialFogParameters2;
		Out.ExponentialFogColorParameter = FogStructISR.ExponentialFogColorParameter;
		Out.ExponentialFogParameters3 = FogStructISR.ExponentialFogParameters3;
		Out.SkyAtmosphereAmbientContributionColorScale = FogStructISR.SkyAtmosphereAmbientContributionColorScale;
		Out.InscatteringLightDirection = FogStructISR.InscatteringLightDirection;
		Out.DirectionalInscatteringColor = FogStructISR.DirectionalInscatteringColor;
		Out.SinCosInscatteringColorCubemapRotation = FogStructISR.SinCosInscatteringColorCubemapRotation;
		Out.FogInscatteringTextureParameters = FogStructISR.FogInscatteringTextureParameters;
		Out.EndDistance = FogStructISR.EndDistance;
		Out.ApplyVolumetricFog = FogStructISR.ApplyVolumetricFog;
		Out.VolumetricFogStartDistance = FogStructISR.VolumetricFogStartDistance;
		Out.VolumetricFogNearFadeInDistanceInv = FogStructISR.VolumetricFogNearFadeInDistanceInv;
		Out.VolumetricFogAlbedo = FogStructISR.VolumetricFogAlbedo;
		Out.VolumetricFogPhaseG = FogStructISR.VolumetricFogPhaseG;
		Out.SkyLightCaptureAffectsHeightFogStrength  = FogStruct.SkyLightCaptureAffectsHeightFogStrength;
		Out.SkyLightCaptureAffectsHeightFogRoughness = FogStruct.SkyLightCaptureAffectsHeightFogRoughness;
		Out.SkyLightCaptureAffectsHeightFogMipCount  = FogStruct.SkyLightCaptureAffectsHeightFogMipCount;
		// Intentionally commented out to avoid spir-v generation issue - copying texture types is not supported
		//Out.FogInscatteringColorCubemap = FogStructISR.FogInscatteringColorCubemap;
		//Out.IntegratedLightScattering = FogStructISR.IntegratedLightScattering;
	}
#endif
	return Out;
}

float3 ComputeInscatteringColor(float3 CameraToReceiver, float CameraToReceiverLength, FogStructData FogData, uint EyeIndex)
{
	half3 Inscattering = FogData.ExponentialFogColorParameter.xyz;

#if PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE
	BRANCH
	if (FogData.ExponentialFogParameters3.z > 0)
	{
		float FadeAlpha = saturate(CameraToReceiverLength * FogData.FogInscatteringTextureParameters.x + FogData.FogInscatteringTextureParameters.y);
		float3 CubemapLookupVector = CameraToReceiver;
		// Rotate around Z axis
		CubemapLookupVector.xy = float2(dot(CubemapLookupVector.xy, float2(FogData.SinCosInscatteringColorCubemapRotation.y, -FogData.SinCosInscatteringColorCubemapRotation.x)), dot(CubemapLookupVector.xy, FogData.SinCosInscatteringColorCubemapRotation.xy));
		float3 DirectionalColor; 
		float3 NonDirectionalColor;
// Spir-v does not support copying texture types so FogInscatteringColorCubemap needs to be fetched directly
// from FogStruct or FogStructISR instead of being stored in FogData.
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER
		if(EyeIndex == 0)
		{
#endif
			DirectionalColor = TextureCubeSampleLevel(FogStruct.FogInscatteringColorCubemap, SharedFogInColorScatteringSampler, CubemapLookupVector, 0).xyz;
			NonDirectionalColor = TextureCubeSampleLevel(FogStruct.FogInscatteringColorCubemap, SharedFogInColorScatteringSampler, CubemapLookupVector, FogData.FogInscatteringTextureParameters.z).xyz;
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER
		}
		else
		{
			DirectionalColor = TextureCubeSampleLevel(FogStructISR.FogInscatteringColorCubemap, SharedFogInColorScatteringSampler, CubemapLookupVector, 0).xyz;
			NonDirectionalColor = TextureCubeSampleLevel(FogStructISR.FogInscatteringColorCubemap, SharedFogInColorScatteringSampler, CubemapLookupVector, FogData.FogInscatteringTextureParameters.z).xyz;
		}
#endif
		Inscattering *= lerp(NonDirectionalColor, DirectionalColor, FadeAlpha);
	}
//	else
#endif

#if PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG
	float3 SkyAmbient = FogData.SkyAtmosphereAmbientContributionColorScale.rgb * View.SkyAtmosphereHeightFogContribution.xxx * GetViewDistanceSkyLightColor();
	#if PROJECT_EXPFOG_MATCHES_VFOG
	SkyAmbient *= FogData.VolumetricFogAlbedo;
	#endif
	Inscattering += SkyAmbient;
#endif

	return Inscattering;
}

float3 ComputeInscatteringColor(float3 CameraToReceiver, float CameraToReceiverLength)
{
	return ComputeInscatteringColor(CameraToReceiver, CameraToReceiverLength, GetFogStructData(0), 0);
}

// Calculate the line integral of the ray from the camera to the receiver position through the fog density function
// The exponential fog density function is d = GlobalDensity * exp(-HeightFalloff * z)
float CalculateLineIntegralShared(float FogHeightFalloff, float RayDirectionZ, float RayOriginTerms)
{
	float Falloff = max(-127.0f, FogHeightFalloff * RayDirectionZ);    // if it's lower than -127.0, then exp2() goes crazy in OpenGL's GLSL.
	float LineIntegral = ( 1.0f - exp2(-Falloff) ) / Falloff;
	float LineIntegralTaylor = log(2.0) - ( 0.5 * Pow2( log(2.0) ) ) * Falloff;		// Taylor expansion around 0
	
	return RayOriginTerms * ( abs(Falloff) > FLT_EPSILON2 ? LineIntegral : LineIntegralTaylor );
}

float PreComputeFogOriginFactor(float OriginHeight, float FogHeight, float FogFalloff, float FogDensity)
{
	// Adapt the precomputed fog parameter to the overriden view origin. See FSceneRenderer::InitFogConstants.
	const float CollapsedFogParameterPower = clamp(
		-FogFalloff * (OriginHeight - FogHeight),
		-126.f + 1.f, // min and max exponent values for IEEE floating points (http://en.wikipedia.org/wiki/IEEE_floating_point)
		+127.f - 1.f
	);
	return FogDensity * pow(2.0f, CollapsedFogParameterPower);
}

// @param WorldPositionRelativeToCamera = WorldPosition - InCameraPosition. It is the vector from the camera to the world position to shade (independent from TranslatedPosition).
half4 GetExponentialHeightFog(float3 WorldPositionRelativeToCamera, float ExcludeDistance, uint EyeIndex, ViewState InView, bool bOverrideOrigin = false, float3 OverrideRayStartRelativeToCamera=0.0f.xxx, float3 OverrideRayDir = 0.0f.xxx, float OverrideRayLength = 0.0f)
{
	FogStructData FogData = GetFogStructData(EyeIndex);

	float3 WorldCameraOrigin = DFHackToFloat(InView.WorldCameraOrigin);
	float3 CameraToReceiver = WorldPositionRelativeToCamera;

	// Statically removed by the compiler when not requested.
	if (bOverrideOrigin)
	{
		// Adapt fog precomputed parameter, as done on CPU, to a new origin that is not the View (for instance when tracing in lumen with any origin) but a traced ray anywhere in the world.
		WorldCameraOrigin += OverrideRayStartRelativeToCamera;
		CameraToReceiver   = OverrideRayDir * OverrideRayLength;	
		FogData.ExponentialFogParameters.x  = PreComputeFogOriginFactor(WorldCameraOrigin.z, FogData.ExponentialFogParameters3.y, FogData.ExponentialFogParameters.y,  FogData.ExponentialFogParameters3.x);
		FogData.ExponentialFogParameters2.x = PreComputeFogOriginFactor(WorldCameraOrigin.z, FogData.ExponentialFogParameters2.w, FogData.ExponentialFogParameters2.y, FogData.ExponentialFogParameters2.z);
	}
	
	const half MinFogOpacity = FogData.ExponentialFogColorParameter.w;
	const float MaxWorldObserverHeight = FogData.ExponentialFogParameters.z;

	const float3 WorldObserverOrigin = float3(WorldCameraOrigin.xy, min(WorldCameraOrigin.z, MaxWorldObserverHeight)); // Clamp z to max height

	// Apply end fog distance from view projected on the XY plane.
	const float CameraToReceiverLenXYSqr = dot(CameraToReceiver.xy, CameraToReceiver.xy);
	if (FogData.EndDistance > 0.0f && CameraToReceiverLenXYSqr > (FogData.EndDistance * FogData.EndDistance))
	{
		CameraToReceiver *= FogData.EndDistance / sqrt(max(1.0, CameraToReceiverLenXYSqr));
	}

	CameraToReceiver.z += WorldCameraOrigin.z - WorldObserverOrigin.z; // Compensate this vector for clamping the observer height
	float CameraToReceiverLengthSqr = dot(CameraToReceiver, CameraToReceiver);
	float CameraToReceiverLengthInv = rsqrt(max(CameraToReceiverLengthSqr, 0.00000001f));
	float CameraToReceiverLength = CameraToReceiverLengthSqr * CameraToReceiverLengthInv;
	half3 CameraToReceiverNormalized = CameraToReceiver * CameraToReceiverLengthInv;

	float RayOriginTerms = FogData.ExponentialFogParameters.x;
	float RayOriginTermsSecond = FogData.ExponentialFogParameters2.x;
	float RayLength = CameraToReceiverLength;
	float RayDirectionZ = CameraToReceiver.z;

#if USE_GLOBAL_CLIP_PLANE

	BRANCH
	// While rendering a planar reflection with a clip plane, we must compute analytical fog using a camera path starting from the plane, rather than the virtual camera origin
	if (dot(InView.GlobalClippingPlane.xyz, 1) > 0.0f)
	{
		float CameraOriginPlaneDistance = dot(InView.GlobalClippingPlane, float4(WorldObserverOrigin + DFHackToFloat(InView.PreViewTranslation), 1));
		float PlaneIntersectionTime = -CameraOriginPlaneDistance / dot(CameraToReceiver, InView.GlobalClippingPlane.xyz);

		// Only modify the start distance if the reflection plane is between the camera and receiver point
		if (PlaneIntersectionTime > 0 && PlaneIntersectionTime < 1)
		{
			ExcludeDistance = max(ExcludeDistance, PlaneIntersectionTime * CameraToReceiverLength);
		}
	}

#endif

	// Factor in StartDistance
#if PERMUTATION_SUPPORT_FOG_START_DISTANCE
	ExcludeDistance = max(ExcludeDistance, FogData.ExponentialFogParameters.w);
	if (ExcludeDistance > 0)
	{
		float ExcludeIntersectionTime = ExcludeDistance * CameraToReceiverLengthInv;
		float CameraToExclusionIntersectionZ = ExcludeIntersectionTime * CameraToReceiver.z;
		float ExclusionIntersectionZ = WorldObserverOrigin.z + CameraToExclusionIntersectionZ;
		float ExclusionIntersectionToReceiverZ = CameraToReceiver.z - CameraToExclusionIntersectionZ;

		// Calculate fog off of the ray starting from the exclusion distance, instead of starting from the camera
		RayLength = (1.0f - ExcludeIntersectionTime) * CameraToReceiverLength;
		RayDirectionZ = ExclusionIntersectionToReceiverZ;

		float Exponent = max(-127.0f, FogData.ExponentialFogParameters.y * (ExclusionIntersectionZ - FogData.ExponentialFogParameters3.y));
		RayOriginTerms = FogData.ExponentialFogParameters3.x * exp2(-Exponent);
		
		float ExponentSecond = max(-127.0f, FogData.ExponentialFogParameters2.y * (ExclusionIntersectionZ - FogData.ExponentialFogParameters2.w));
		RayOriginTermsSecond = FogData.ExponentialFogParameters2.z * exp2(-ExponentSecond);
	}
#endif

	// Calculate the "shared" line integral (this term is also used for the directional light inscattering) by adding the two line integrals together (from two different height falloffs and densities)
	float ExponentialHeightLineIntegralShared = CalculateLineIntegralShared(FogData.ExponentialFogParameters.y, RayDirectionZ, RayOriginTerms);
#if PERMUTATION_SUPPORT_FOG_SECOND_TERM
	ExponentialHeightLineIntegralShared += CalculateLineIntegralShared(FogData.ExponentialFogParameters2.y, RayDirectionZ, RayOriginTermsSecond);
#endif

	float ExponentialHeightLineIntegral = ExponentialHeightLineIntegralShared * RayLength;

	half3 InscatteringColor = ComputeInscatteringColor(CameraToReceiver, CameraToReceiverLength, FogData, EyeIndex);
	half3 DirectionalInscattering = 0;

#if PERMUTATION_SUPPORT_FOG_DIRECTIONAL_LIGHT_INSCATTERING
	// if InscatteringLightDirection.w is negative then it's disabled, otherwise it holds directional inscattering start distance
	BRANCH
	if (FogData.InscatteringLightDirection.w >= 0
	#if PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE
		&& FogData.ExponentialFogParameters3.z == 0
	#endif
	)
	{
#if PROJECT_SUPPORT_SKY_ATMOSPHERE_AFFECTS_HEIGHFOG
		const float UniformPhaseFunction = 1.0f / (4.0f * PI);
		half3 DirectionalInscatteringColor;
		half3 DirectionalLightInscattering;

		// No need to test View.AtmosphereLightIlluminanceOnGroundPostTransmittance[0].a because InscatteringLightDirection.w above is doing the same test already.
		DirectionalInscatteringColor = FogData.DirectionalInscatteringColor.xyz + InView.SkyAtmosphereHeightFogContribution * InView.AtmosphereLightIlluminanceOnGroundPostTransmittance[0].rgb;
		DirectionalLightInscattering = DirectionalInscatteringColor 
	#if PROJECT_EXPFOG_MATCHES_VFOG
			* HenyeyGreensteinPhase(FogData.VolumetricFogPhaseG, dot(-CameraToReceiverNormalized, InView.AtmosphereLightDirection[0].xyz));
	#else
			* pow(saturate(dot(CameraToReceiverNormalized, InView.AtmosphereLightDirection[0].xyz)), FogData.DirectionalInscatteringColor.w) * UniformPhaseFunction;
	#endif

		if (InView.AtmosphereLightIlluminanceOnGroundPostTransmittance[1].a > 0.0f) // Skip DirectionalInscatteringColor on the second light when disabled.
		{
			DirectionalInscatteringColor = FogData.DirectionalInscatteringColor.xyz + InView.SkyAtmosphereHeightFogContribution * InView.AtmosphereLightIlluminanceOnGroundPostTransmittance[1].rgb;
			DirectionalLightInscattering += DirectionalInscatteringColor 
		#if PROJECT_EXPFOG_MATCHES_VFOG
				* HenyeyGreensteinPhase(FogData.VolumetricFogPhaseG, dot(-CameraToReceiverNormalized, InView.AtmosphereLightDirection[1].xyz));
		#else
				* pow(saturate(dot(CameraToReceiverNormalized, InView.AtmosphereLightDirection[1].xyz)), FogData.DirectionalInscatteringColor.w) * UniformPhaseFunction;
		#endif
		}
#else
		// Setup a cosine lobe around the light direction to approximate inscattering from the directional light off of the ambient haze;
		half3 DirectionalLightInscattering = FogData.DirectionalInscatteringColor.xyz
	#if PROJECT_EXPFOG_MATCHES_VFOG
			* HenyeyGreensteinPhase(FogData.VolumetricFogPhaseG, dot(-CameraToReceiverNormalized, InView.AtmosphereLightDirection[0].xyz));
	#else
			* pow(saturate(dot(CameraToReceiverNormalized, FogData.InscatteringLightDirection.xyz)), FogData.DirectionalInscatteringColor.w);
	#endif
#endif

		// Calculate the line integral of the eye ray through the haze, using a special starting distance to limit the inscattering to the distance
		float DirectionalInscatteringStartDistance = PROJECT_EXPFOG_MATCHES_VFOG ? 0.0 : FogData.InscatteringLightDirection.w;
		float DirExponentialHeightLineIntegral = ExponentialHeightLineIntegralShared * max(RayLength - DirectionalInscatteringStartDistance, 0.0f);
		// Calculate the amount of light that made it through the fog using the transmission equation
		half DirectionalInscatteringFogFactor = saturate(exp2(-DirExponentialHeightLineIntegral));
		// Final inscattering from the light
		DirectionalInscattering = DirectionalLightInscattering * (1 - DirectionalInscatteringFogFactor);
	}
#endif

#if SUPPORTS_SKYLIGHTCAPTURE_AFFECTS_HEIGHTFOGINSCATTERING 
	if (FogData.SkyLightCaptureAffectsHeightFogStrength > 0.0f && View.RenderingReflectionCaptureMask == 0)
	{
		float3 CubemapLookupVector = CameraToReceiver;
		float AbsoluteSpecularMip = ComputeReflectionCaptureMipFromRoughness(FogData.SkyLightCaptureAffectsHeightFogRoughness, FogData.SkyLightCaptureAffectsHeightFogMipCount);
		float3 SkyLightInscatteringColor = View.SkyLightColor.rgb * TextureCubeSampleLevel(FogStruct.FogInscatteringColorCubemap, SharedFogInColorScatteringSampler, CubemapLookupVector, AbsoluteSpecularMip).xyz;

		// Lerp based on influence to not account for twice the energy.
		// No applied when rendering capture otherwise it could cause infinite feedback loop and luminance/exposure explosion.
		// It is not applied on volumetric fog or local fog volumes since:
		//  - It is too complex to manage with lumen GI mixed with occluded skylight in a single color.
		//  - Also having that kind of scattering on VFog or LFV could lead to sky light in scattering leaking indoors. (it can happen with height fog but it usually has a larger start distance).
		DirectionalInscattering = lerp(DirectionalInscattering, 0.0f, FogData.SkyLightCaptureAffectsHeightFogStrength);
		InscatteringColor = lerp(InscatteringColor, SkyLightInscatteringColor, FogData.SkyLightCaptureAffectsHeightFogStrength);
	}
#endif

#if PROJECT_EXPFOG_MATCHES_VFOG
	DirectionalInscattering *= FogData.VolumetricFogAlbedo;
#endif

	// Calculate the amount of light that made it through the fog using the transmission equation
	half ExpFogFactor = max(saturate(exp2(-ExponentialHeightLineIntegral)), MinFogOpacity);

	FLATTEN
	if (FogData.ExponentialFogParameters3.w > 0 && CameraToReceiverLength > FogData.ExponentialFogParameters3.w)
	{
		ExpFogFactor = 1;
		DirectionalInscattering = 0;
	}

	// Fog color is unused when additive / modulate blend modes are active.
	#if (MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE) && !FOG_MATERIALBLENDING_OVERRIDE
		half3 FogColor = 0.0;
	#else
		half3 FogColor = (InscatteringColor) * (1 - ExpFogFactor) + DirectionalInscattering;
	#endif

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsExponentialFogHoldout(View.EnvironmentComponentsFlags) && View.bPrimitiveAlphaHoldoutEnabled)
	{
		FogColor.rgb *= 0;
	}
#endif

	return half4(FogColor, ExpFogFactor);
}

// @param WorldPositionRelativeToCamera = WorldPosition - InCameraPosition. It is the vector from the camera to the world position to shade (independent from TranslatedPosition).
half4 GetExponentialHeightFog(float3 WorldPositionRelativeToCamera, float ExcludeDistance)
{
	return GetExponentialHeightFog(WorldPositionRelativeToCamera, ExcludeDistance, 0, GetPrimaryView());
}

// @param WorldPositionRelativeToCamera = WorldPosition - InCameraPosition. It is the vector from the camera to the world position to shade (independent from TranslatedPosition).
half4 CalculateHeightFog(float3 WorldPositionRelativeToCamera, uint EyeIndex, ViewState InView)
{
	if (InView.RenderingReflectionCaptureMask == 0.0f && !IsExponentialFogRenderedInMain(InView.EnvironmentComponentsFlags))
	{
		return float4(0.0f, 0.0f, 0.0f, 1.0f);
	}

	float ExcludeDistance = 0;
#if PERMUTATION_SUPPORT_VOLUMETRIC_FOG
	float CosAngle = dot(normalize(WorldPositionRelativeToCamera), InView.ViewForward);

	#if NEEDS_BASEPASS_VERTEX_FOGGING
		// When vertex fogging is used, we need to make sure that the fog applied in base pass and the fog applied in pixel match as much as possible.
		// We however do not want the popping resulting from the per pixel math below due to undefined behavior when ZDepth=0, or we do not want to have full fog InvCosAngle = 0 when a vertex is behind the camera projection plane.
		// Per vertex fog cannot match volumetric fog blending plane anyway since it is simply lerp over triangles. So in this case we simply exlude the fog from the sphere around the camera with radius VolumetricFogMaxDistance.
		ExcludeDistance = InView.VolumetricFogMaxDistance;
	#else
		// Volumetric fog covers up to MaxDistance along ViewZ, exclude analytical fog from this range.
		// So when we evaluate per pixel fog, we can compute the exact distance along the view ray to not have fog integrated for as (StartDistance / CosAngle).
		// This is basically the distance along the view ray to the point on the start distance plane, which is parallel to the camera plane (not a sphere around the camera)
		// That is how current height fog works and blends in from no fog before start distance, or volumetric fog, into height fog.
		float InvCosAngle = (CosAngle > FLT_EPSILON) ? rcp(CosAngle) : 0; // If behind the camera we do remove fog, but that should not happen for per pixel fog.
		ExcludeDistance = InView.VolumetricFogMaxDistance * InvCosAngle;
	#endif
#endif

	half4 FogInscatteringAndOpacity = GetExponentialHeightFog(WorldPositionRelativeToCamera, ExcludeDistance, EyeIndex, InView);
	return FogInscatteringAndOpacity;
}

// @param WorldPositionRelativeToCamera = WorldPosition - InCameraPosition. It is the vector from the camera to the world position to shade (independent from TranslatedPosition).
half4 CalculateHeightFog(float3 WorldPositionRelativeToCamera)
{
	return CalculateHeightFog(WorldPositionRelativeToCamera, 0, GetPrimaryView());
}

float4 SampleVolumetricFog(float3 VolumeUV)
{
	return Texture3DSampleLevel(FogStruct.IntegratedLightScattering, SharedIntegratedLightScatteringSampler, VolumeUV, 0);
}

#if !defined(PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY)
// Only passes defining this value can use higher quality filtering. Forward shading won't for instance.
#define PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY 0
#endif

float4 SampleVolumetricFogFiltered(float3 VolumeUV)
{
#if PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY > 0
		//Prerequisities for Tri & Bicubic
		float3 PixelPos = VolumeUV * View.VolumetricFogGridSize + 0.5f;
		uint3 PixelPosFloor = floor(PixelPos);
		float3 Gradient = frac(PixelPos);
		float3 OneMinusGradient = 1.0f - Gradient;
			
		float3 w0 = 1.0 / 6.0 * OneMinusGradient * OneMinusGradient * OneMinusGradient;
		float3 w1 = 2.0 / 3.0 - 0.5 * Gradient * Gradient * (2.0 - Gradient);
		float3 w2 = 2.0 / 3.0 - 0.5 * OneMinusGradient * OneMinusGradient * (2.0 - OneMinusGradient);
		float3 w3 = 1.0 / 6.0 * Gradient * Gradient * Gradient;

		float3 g0 = w0 + w1;
		float3 g1 = w2 + w3;
		float3 h0 = View.VolumetricFogInvGridSize * ((w1 / g0) - 1.5 + PixelPosFloor);
		float3 h1 = View.VolumetricFogInvGridSize * ((w3 / g1) + 0.5 + PixelPosFloor);
		h1.xy = min(h1.xy, View.VolumetricFogUVMax);
		h0.z = VolumeUV.z;
#endif
	
#if PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY == 2
		//Tricubic
		float4 tex000 = SampleVolumetricFog(h0);
		float4 tex100 = SampleVolumetricFog(float3(h1.x, h0.y, h0.z));
		tex000 = lerp(tex100, tex000, g0.x);

		float4 tex010 = SampleVolumetricFog(float3(h0.x, h1.y, h0.z));
		float4 tex110 = SampleVolumetricFog(float3(h1.x, h1.y, h0.z));
		tex010 = lerp(tex110, tex010, g0.x);
		tex000 = lerp(tex010, tex000, g0.y);

		float4 tex001 = SampleVolumetricFog(float3(h0.x, h0.y, h1.z));
		float4 tex101 = SampleVolumetricFog(float3(h1.x, h0.y, h1.z));
		tex001 = lerp(tex101, tex001, g0.x);

		float4 tex011 = SampleVolumetricFog(float3(h0.x, h1.y, h1.z));
		float4 tex111 = SampleVolumetricFog(h1);
		tex011 = lerp(tex111, tex011, g0.x);
		tex001 = lerp(tex011, tex001, g0.y);

		return lerp(tex001, tex000, g0.z);	
#elif PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY == 1
		//Bicubic
		float4 tex000 = SampleVolumetricFog(h0);
		float4 tex100 = SampleVolumetricFog(float3(h1.x, h0.y, h0.z));
		tex000 = lerp(tex100, tex000, g0.x);

		float4 tex010 = SampleVolumetricFog(float3(h0.x, h1.y, h0.z));
		float4 tex110 = SampleVolumetricFog(float3(h1.x, h1.y, h0.z));
		tex010 = lerp(tex110, tex010, g0.x);
	
		return lerp(tex010, tex000, g0.y);	
#else
		//Trilinear
	return SampleVolumetricFog(VolumeUV);
#endif

	// Idea: stochastic sampling into noise resolved by TSR to make higher order filtering cheaper.
}

float4 CombineVolumetricFog(float4 GlobalFog, float3 VolumeUV, uint EyeIndex, float SceneDepth, ViewState InView)
{
	float4 VolumetricFogLookup = float4(0, 0, 0, 1);
	
	if (InView.RenderingReflectionCaptureMask == 0.0f && !IsExponentialFogRenderedInMain(InView.EnvironmentComponentsFlags))
	{
		return VolumetricFogLookup;
	}

#if PERMUTATION_SUPPORT_VOLUMETRIC_FOG
	float VolFogStartDistance = 0.0f;
	if (FogStruct.ApplyVolumetricFog > 0)
	{
	#if INSTANCED_STEREO || (MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE && PIXELSHADER)
		if (EyeIndex == 0)
		{
			VolFogStartDistance = FogStruct.VolumetricFogStartDistance;
			VolumetricFogLookup = Texture3DSampleLevel(FogStruct.IntegratedLightScattering, SharedIntegratedLightScatteringSampler, VolumeUV, 0);
		}
		else
		{
			VolFogStartDistance = FogStructISR.VolumetricFogStartDistance;
			VolumetricFogLookup = Texture3DSampleLevel(FogStructISR.IntegratedLightScattering, SharedIntegratedLightScatteringSampler, VolumeUV, 0);
		}
	#else
		VolFogStartDistance = FogStruct.VolumetricFogStartDistance;
		VolumetricFogLookup = SampleVolumetricFogFiltered(VolumeUV);
	#endif

		// IntegratedLightScattering is pre-exposed, remove pre exposure now so that it can correctly be applied later
		VolumetricFogLookup.rgb *= InView.OneOverPreExposure;
	}

	// Do not apply the Froxel volumetric texture in front of the fog start distance. (the soft fading occur in FinalIntegrationCS).
	// We go with a quickly increasing step function because the soft fade in from start distance occurs in FinalIntegrationCS.
	VolumetricFogLookup = lerp(float4(0, 0, 0, 1), VolumetricFogLookup, saturate((SceneDepth - VolFogStartDistance) * 100000000.0f));
#endif

	// Visualize depth distribution
	//VolumetricFogLookup.rgb += .1f * frac(min(ZSlice, 1.0f) / View.VolumetricFogInvGridSize.z);

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsExponentialFogHoldout(InView.EnvironmentComponentsFlags) && InView.bPrimitiveAlphaHoldoutEnabled)
	{
		VolumetricFogLookup.rgb *= 0;
		GlobalFog.rgb *= 0;
	}
#endif

	return float4(VolumetricFogLookup.rgb + GlobalFog.rgb * VolumetricFogLookup.a, VolumetricFogLookup.a * GlobalFog.a);
}

float4 CombineVolumetricFog(float4 GlobalFog, float3 VolumeUV, uint EyeIndex, float SceneDepth)
{
	return CombineVolumetricFog(GlobalFog, VolumeUV, EyeIndex, SceneDepth, GetPrimaryView());
}

float ComputeNormalizedZSliceFromDepth(float SceneDepth, ViewState InView)
{
	return log2(SceneDepth * InView.VolumetricFogGridZParams.x + InView.VolumetricFogGridZParams.y) * InView.VolumetricFogGridZParams.z * InView.VolumetricFogInvGridSize.z;
}

float ComputeNormalizedZSliceFromDepth(float SceneDepth)
{
	return ComputeNormalizedZSliceFromDepth(SceneDepth, GetPrimaryView());
}

float3 ComputeVolumeUVFromNDC(float4 NDCPosition, ViewState InView)
{
	NDCPosition.xy /= NDCPosition.w;
	float3 VolumeUV = float3(NDCPosition.xy * float2(.5f, -.5f) + .5f, ComputeNormalizedZSliceFromDepth(NDCPosition.w, InView));
	return min(VolumeUV * float3(InView.VolumetricFogScreenToResourceUV.xy, 1.0), float3(InView.VolumetricFogUVMax, 1.0));
}

float3 ComputeVolumeUVFromNDC(float4 NDCPosition)
{
	return ComputeVolumeUVFromNDC(NDCPosition, GetPrimaryView());
}

float3 ComputeVolumeUVFromTranslatedPos(float3 TranslatedWorldPosition, float4x4 TranslatedWorldToClip, ViewState InView)
{
	float4 NDCPosition = mul(float4(TranslatedWorldPosition, 1), TranslatedWorldToClip);
	return ComputeVolumeUVFromNDC(NDCPosition, InView);
}
float3 ComputeVolumeUVFromTranslatedPos(float3 TranslatedWorldPosition, float4x4 TranslatedWorldToClip)
{
	return ComputeVolumeUVFromTranslatedPos(TranslatedWorldPosition, TranslatedWorldToClip, GetPrimaryView());
}
float3 ComputeVolumeUV_DEPRECATED(float3 WorldPosition, float4x4 WorldToClip)
{
	float4 NDCPosition = mul(float4(WorldPosition, 1), WorldToClip);
	return ComputeVolumeUVFromNDC(NDCPosition);
}
float3 ComputeVolumeUV(FDFVector3 WorldPosition, FDFInverseMatrix WorldToClip, ViewState InView)
{
	float4 NDCPosition = DFMultiplyDemote(MakeDFVector(WorldPosition, 1.0f), WorldToClip);
	return ComputeVolumeUVFromNDC(NDCPosition, InView);
}
float3 ComputeVolumeUV(FDFVector3 WorldPosition, FDFInverseMatrix WorldToClip)
{
	return ComputeVolumeUV(WorldPosition, WorldToClip, GetPrimaryView());
}

float3 ComputeHistoryVolumeUVFromNDC(float4 NDCPosition)
{
	NDCPosition.xy /= NDCPosition.w;
	float3 VolumeUV = float3(NDCPosition.xy * float2(.5f, -.5f) + .5f, ComputeNormalizedZSliceFromDepth(NDCPosition.w));
	return min(VolumeUV * float3(View.VolumetricFogViewGridUVToPrevViewRectUV.xy * View.VolumetricFogPrevViewGridRectUVToResourceUV.xy, 1.0), float3(View.VolumetricFogPrevUVMaxForTemporalBlend, 1.0));
}
float3 ComputeHistoryVolumeUVFromTranslatedPos(float3 TranslatedWorldPosition, float4x4 TranslatedWorldToClip)
{
	float4 NDCPosition = mul(float4(TranslatedWorldPosition, 1), TranslatedWorldToClip);
	return ComputeHistoryVolumeUVFromNDC(NDCPosition);
}
float3 ComputeHistoryVolumeUV(FDFVector3 WorldPosition, FDFInverseMatrix WorldToClip)
{
	float4 NDCPosition = DFMultiplyDemote(MakeDFVector(WorldPosition, 1.0f), WorldToClip);
	return ComputeHistoryVolumeUVFromNDC(NDCPosition);
}
