// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightFogPixelShader.usf: Scene fogging pixel shader.
=============================================================================*/

#include "Common.ush"
#include "SceneTexturesCommon.ush"
#include "SHCommon.ush"
#include "HeightFogCommon.ush"
#include "Random.ush"

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

#if PERMUTATION_SAMPLE_SKYATMOSPHERE_ON_OPAQUE && PROJECT_SUPPORT_SKY_ATMOSPHERE
#include "SkyAtmosphereCommon.ush"
#endif

#include "FogScreenSpaceScattering.ush"

#ifdef RenderSeparateCompositionTexturesCS
#define RENDER_FOG_COMP_TEXTURE_CS 1
#else
#define RENDER_FOG_COMP_TEXTURE_CS 0
#endif

// This is not defined for RenderSeparateCompositionTexturesCS
#ifndef PERMUTATION_SAMPLE_FOG_ON_CLOUDS
#define PERMUTATION_SAMPLE_FOG_ON_CLOUDS 0
#endif

float UpsampleJitterMultiplier;

float bOnlyOnRenderedOpaque;
uint bUseWaterDepthTexture;
uint bPropagateAlpha;

float4 WaterDepthTextureMinMaxUV;
float4 OcclusionTextureMinMaxUV;

Texture2D OcclusionTexture;
SamplerState OcclusionSampler;

Texture2D WaterDepthTexture;
SamplerState WaterDepthSampler;










#if PERMUTATION_SAMPLE_FOG_ON_CLOUDS
Texture2D<float> SrcCloudDepthTexture;
SamplerState SrcCloudDepthSampler;
Texture2D<float4> SrcCloudViewTexture;
SamplerState SrcCloudViewSampler;
#endif

#if RENDER_FOG_COMP_TEXTURE_CS
Texture2D<float> FogSeparateCompositionDepth;
RWTexture2D<float4> FogTexture0UAV;
RWTexture2D<float2> FogTexture1UAV;
float2 FogSeparateCompositionViewResolution;
float2 FogSeparateCompositionViewRectMin;
float FogSeparateCompositionResolutionDivider;
Texture2D<float4> InputSceneColorTexture;
SamplerState InputSceneColorTextureSampler;
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RenderSeparateCompositionTexturesCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy > FogSeparateCompositionViewResolution))
	{
		return;
	}
	float4 SVPos			= float4((float2(ThreadId.xy) + 0.5f) * FogSeparateCompositionResolutionDivider + FogSeparateCompositionViewRectMin.xy, 0.0f, 1.0f) + float4(0.5f, 0.5f, 0.0, 0.0);
	float2 TexCoord			= (float2(ThreadId.xy) + 0.5f) * View.BufferSizeAndInvSize.zw;
	float2 TexCoordSceneTex	= (float2(FogSeparateCompositionViewRectMin + ThreadId.xy) * FogSeparateCompositionResolutionDivider + 0.5f) * View.BufferSizeAndInvSize.zw;
	float4 ScreenPosition	= SvPositionToScreenPosition(float4(SVPos.xy, 0.5, 1));
	float3 ScreenVector		= ScreenVectorFromScreenRect(float4(ScreenPosition.xy, 1, 0));
	float4 OutColor			= float4(0.0f, 0.0f, 0.0f, 1.0f);
#else // RENDER_FOG_COMP_TEXTURE_CS
void ExponentialPixelMain(
	float2 TexCoord : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	float2 TexCoordSceneTex = TexCoord;
#endif // RENDER_FOG_COMP_TEXTURE_CS

	float SceneDepth;
	float DeviceZ;
#if RENDER_FOG_COMP_TEXTURE_CS

	DeviceZ = FogSeparateCompositionDepth[ThreadId.xy + FogSeparateCompositionViewRectMin];
	SceneDepth = ConvertFromDeviceZ(DeviceZ);

#else // RENDER_FOG_COMP_TEXTURE_CS
#if PERMUTATION_SAMPLE_FOG_ON_CLOUDS

	float2 PixPos = SVPos.xy;
	const float4 CloudLuminanceTransmittance = SrcCloudViewTexture.Load(int3(PixPos, 0));
	const float CloudCoverage = 1.0f - CloudLuminanceTransmittance.a;
	if (CloudLuminanceTransmittance.a > 0.999)
	{
		OutColor = float4(0.0f, 0.0f, 0.0f, 1.0f);
		return;
	}

	const float CloudDepthKm = SrcCloudDepthTexture.Load(int3(PixPos, 0)).r;
	SceneDepth = CloudDepthKm * KILOMETER_TO_CENTIMETER;
	DeviceZ = ConvertToDeviceZ(SceneDepth); // Warning: for simplicity, we use DeviceZ as world distance in kilometer when SAMPLE_ATMOSPHERE_ON_CLOUDS. See special case in IntegrateSingleScatteredLuminance.

#else
	if (bUseWaterDepthTexture)
	{
		float2 SampleUV = clamp(TexCoord, WaterDepthTextureMinMaxUV.xy, WaterDepthTextureMinMaxUV.zw);
		DeviceZ = Texture2DSampleLevel(WaterDepthTexture, WaterDepthSampler, SampleUV, 0).r;
		SceneDepth = ConvertFromDeviceZ(DeviceZ);
	}
	else
	{
	#if SCENE_TEXTURES_DISABLED
		float SceneDepth = SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
		DeviceZ = ConvertToDeviceZ(SceneDepth);
	#else
		DeviceZ = Texture2DSampleLevel(SceneTexturesStruct.SceneDepthTexture, SceneTexturesStruct_SceneDepthTextureSampler, TexCoord, 0).r;
		SceneDepth = ConvertFromDeviceZ(DeviceZ); // Fetch the depth buffer Z / W value, solve for W
	#endif
	}
#endif
#endif // RENDER_FOG_COMP_TEXTURE_CS

	bool bIsRendered = (DeviceZ != 0.0);

	float3 WorldPositionRelativeToCamera = ScreenVector.xyz * SceneDepth;

	float ZSlice = log2(SceneDepth * View.VolumetricFogGridZParams.x + View.VolumetricFogGridZParams.y) * View.VolumetricFogGridZParams.z * View.VolumetricFogInvGridSize.z;
	
	uint3 Rand16Bits = Rand3DPCG16(int3(SVPos.xy, View.StateFrameIndexMod8));
	float3 Rand3D = (float3(Rand16Bits) / float(uint(0xffff))) * 2.0f - 1.0f;
	float3 CellOffset = UpsampleJitterMultiplier * Rand3D;

	float3 VolumeUV = float3(((SVPos.xy - View.ViewRectMin.xy) + CellOffset.xy) * View.VolumetricFogSVPosToVolumeUV, ZSlice);
	VolumeUV.xy = min(VolumeUV.xy, View.VolumetricFogUVMax);

	float4 HeightFogInscatteringAndOpacity = CalculateHeightFog(WorldPositionRelativeToCamera);

#if PERMUTATION_SAMPLE_SKYATMOSPHERE_ON_OPAQUE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	// Sky pixels are rendererd with a sky dome, when tracing per pixel, we do not go that CS path used to render half resolution fog.
	if (DeviceZ != FarDepthValue && PrimaryView.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f)
	{
		const float3 TranslatedWorldPosition = WorldPositionRelativeToCamera + PrimaryView.TranslatedWorldCameraOrigin;
		const float3 WorldPosition = DFHackToFloat(DFFastSubtract(DFPromote(TranslatedWorldPosition), PrimaryView.PreViewTranslation));
		const float4 NDCPosition = mul(float4(WorldPosition.xyz, 1), DFHackToFloat(PrimaryView.WorldToClip));

		// Sample the aerial perspective
		HeightFogInscatteringAndOpacity = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			PrimaryView.RealTimeReflectionCapture, PrimaryView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			NDCPosition, WorldPositionRelativeToCamera * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler,
			PrimaryView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			PrimaryView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			PrimaryView.SkyAtmosphereAerialPerspectiveStartDepthKm,
			PrimaryView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			PrimaryView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			PrimaryView.OneOverPreExposure, HeightFogInscatteringAndOpacity);
	}
#endif // PERMUTATION_SAMPLE_SKYATMOSPHERE_ON_OPAQUE

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
	uint2 TilePos = uint2(SVPos.xy - View.ViewRectMin.xy) / LFVTilePixelSize.xx;
	float4 LFVContribution = GetLFVContribution(PrimaryView, TilePos, WorldPositionRelativeToCamera);
	HeightFogInscatteringAndOpacity = float4(LFVContribution.rgb + HeightFogInscatteringAndOpacity.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndOpacity.a);
#endif

	float4 FogInscatteringAndOpacity = CombineVolumetricFog(HeightFogInscatteringAndOpacity, VolumeUV, 0, SceneDepth);
	float2 OcclusionTexCoord = clamp(TexCoordSceneTex, OcclusionTextureMinMaxUV.xy, OcclusionTextureMinMaxUV.zw);
	float LightShaftMask = Texture2DSample(OcclusionTexture, OcclusionSampler, OcclusionTexCoord).x;
	FogInscatteringAndOpacity.rgb *= LightShaftMask;

	FogInscatteringAndOpacity.rgb *= View.PreExposure;


#if PERMUTATION_SAMPLE_FOG_ON_CLOUDS && !RENDER_FOG_COMP_TEXTURE_CS

	// Reduce cloud luminance according to the fog transmittance and add the fog in scattering luminance according to the cloud coverage.
	OutColor.rgb = CloudLuminanceTransmittance.rgb* FogInscatteringAndOpacity.a + FogInscatteringAndOpacity.rgb * CloudCoverage;
	// Coverage of the cloud layer itself does not change.
	OutColor.a   = CloudLuminanceTransmittance.a;

#else // PERMUTATION_SAMPLE_FOG_ON_CLOUDS

	if (bOnlyOnRenderedOpaque > 0.0 && !bIsRendered)
	{
		FogInscatteringAndOpacity.rgb = 0;
		FogInscatteringAndOpacity.a = 1;
	}
	
	FLATTEN
	if (bPropagateAlpha && !RENDER_FOG_COMP_TEXTURE_CS)
	{
		// We need to output coverage so that we can potentially apply alpha hold out blending. See RenderViewFog.
		OutColor = RETURN_COLOR(float4(FogInscatteringAndOpacity.rgb, 1.0 - FogInscatteringAndOpacity.w));
	}
	else
	{
		OutColor = RETURN_COLOR(float4(FogInscatteringAndOpacity.rgb, FogInscatteringAndOpacity.w));
	}
	
#endif // PERMUTATION_SAMPLE_FOG_ON_CLOUDS

#if RENDER_FOG_COMP_TEXTURE_CS

	const float FogTransmittanceToScene = OutColor.a;
	const float SceneDepthCentimeters   = SceneDepth;
	FFogSSSParameters FSSSParams = EvaluateFSSSParameter(FogTransmittanceToScene, SceneDepthCentimeters);

	// Amount of scene color to scater, ignoring the phase function. It is assumed the phase function is part of FSSSSceneColorScatteringAmountScale.
	const float SceneColorToScatterAmount = pow(saturate(FSSSParams.Coverage * FSSSParams.Albedo * FogStruct.FSSSSceneColorScatteringAmountScale), FogStruct.FSSSSceneColorScatteringAmountPower);

	// This is point sampling scene color when using full resolution. At half resolution this is average of 2x2 texels.
	const float3 SceneColorToScatter = FogTransmittanceToScene * SceneColorToScatterAmount * Texture2DSampleLevel(InputSceneColorTexture, InputSceneColorTextureSampler, TexCoordSceneTex, 0).rgb;

	const float FinalFogTransmittanceToScene = FogTransmittanceToScene * saturate(1.0f - SceneColorToScatterAmount);
	const float FogCoverage = saturate(1.0 - FogTransmittanceToScene);

	FogTexture0UAV[ThreadId.xy] = float4(
		OutColor.rgb + SceneColorToScatter,														// Fog and scene luminance to scatter
		1.0f);																					// Cloud transmittance = 1 for now
	FogTexture1UAV[ThreadId.xy] = float2(
		FSSS_W(FSSSParams.Absorption, FSSSParams.Scattering, FSSSParams.SceneDepthMeter),		// W appoxixmating the PSF of the fog
		FinalFogTransmittanceToScene															// All Fog transmitance but clouds.
	);

#endif
}
