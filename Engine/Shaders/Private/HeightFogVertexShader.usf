// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightFogVertexShader.usf: Scene fogging vertex shader.
=============================================================================*/

#include "Common.ush"

void Main(
	in float2 InPosition : ATTRIBUTE0,
	out float2 OutTexCoord : TEXCOORD0,
	out float3 OutScreenVector : TEXCOORD1,
	out float4 OutPosition : SV_POSITION
	)
{
	ResolvedView = ResolveView();

	// screenspace position from vb
	OutPosition = float4(InPosition,0,1);
	// texture coord from vb
	OutTexCoord = InPosition * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;

	// deproject to world space
	OutScreenVector = ScreenVectorFromScreenRect(float4(InPosition,1,0));
}
