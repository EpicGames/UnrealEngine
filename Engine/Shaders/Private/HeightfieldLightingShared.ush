// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightfieldLightingShared.usf
=============================================================================*/

Texture2D HeightfieldTexture;
SamplerState HeightfieldSampler;
Texture2D DiffuseColorTexture;
SamplerState DiffuseColorSampler;
Texture2D VisibilityTexture;
SamplerState VisibilitySampler;
uint NumHeightfields;
Buffer<float4> HeightfieldDescriptions;

// In float4's, must match C++
#define HEIGHTFIELD_DATA_STRIDE 12

float4 GetHeightfieldScaleBias(uint Index, out bool bHasSubsections)
{
	float4 HeightfieldVector = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 0];
	// Sign bit stored presence of subsections
	bHasSubsections = HeightfieldVector.x < 0;
	HeightfieldVector.x = abs(HeightfieldVector.x);
	return HeightfieldVector;
}

float4 GetHeightfieldMinMaxUV(uint Index)
{
	return HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 1];
}

float2 GetHeightfieldSize(uint Index)
{
	return HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 2].xy;
}

float2 GetHeightfieldSizeInv(uint Index)
{
	return HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 2].zw;
}

float3x3 GetWorldToLocalRotT(uint Index)
{
	float4 M0 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 4];
	float4 M1 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 5];
	float4 M2 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 6];

	float3x3 RelativeWorldToLocal = float3x3(M0.xyz, M1.xyz, M2.xyz);
	return RelativeWorldToLocal;
}

FDFInverseMatrix GetWorldToLocal(uint Index)
{
	float3 PositionHigh = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 3].xyz;

	float4 M0 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 4];
	float4 M1 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 5];
	float4 M2 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 6];

	float4x4 RelativeWorldToLocal = transpose(float4x4(M0, M1, M2, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	return MakeDFInverseMatrix4x3(PositionHigh, RelativeWorldToLocal);
}

FDFMatrix GetLocalToWorld(uint Index)
{
	float3 PositionHigh = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 3].xyz;

	float4 M0 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 7];
	float4 M1 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 8];
	float4 M2 = HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 9];

	float4x4 LocalToRelativeWorld = transpose(float4x4(M0, M1, M2, float4(0.0f, 0.0f, 0.0f, 1.0f)));

	return MakeDFMatrix(PositionHigh, LocalToRelativeWorld);
}

float4 GetVisibilityMask(uint Index)
{
	return HeightfieldDescriptions[Index * HEIGHTFIELD_DATA_STRIDE + 10];
}

uint GetHeightfieldGPUSceneInstanceIndex(uint HeightfieldIndex)
{
	float4 V0 = HeightfieldDescriptions[HeightfieldIndex * HEIGHTFIELD_DATA_STRIDE + 11];
	return asuint(V0.x);
}

float GetHeightfieldInvMaxScale(uint HeightfieldIndex)
{
	float4 V0 = HeightfieldDescriptions[HeightfieldIndex * HEIGHTFIELD_DATA_STRIDE + 11];
	return V0.y;
}

float2 GetHeightfieldUV(uint HeightfieldIndex, float2 LocalPosition, out float4 MinMaxHeightfieldUV)
{
	bool bHasSubsections;
	float4 HeightfieldScaleBias = GetHeightfieldScaleBias(HeightfieldIndex, bHasSubsections);
	float2 HeightfieldUV = LocalPosition * HeightfieldScaleBias.xy + HeightfieldScaleBias.zw;
	MinMaxHeightfieldUV = GetHeightfieldMinMaxUV(HeightfieldIndex);

	if (bHasSubsections)
	{
		float2 CenterUV = .5f * (MinMaxHeightfieldUV.xy + MinMaxHeightfieldUV.zw);
		// One row of heights in the heightfield are duplicated between subsections
		// The only number of subsections supported is 2
		// If we are in subsection 0, no adjustment is needed, but if we are in subsection 1 we need to shift up by one texel
		HeightfieldUV += select((HeightfieldUV + .5f * HeightfieldScaleBias.xy) > CenterUV, HeightfieldScaleBias.xy, 0.0);
	}

	return HeightfieldUV;
}

float2 GetHeightfieldUV(uint HeightfieldIndex, float2 LocalPosition)
{
	float4 MinMaxHeightfieldUV;
	return GetHeightfieldUV(HeightfieldIndex, LocalPosition, MinMaxHeightfieldUV);
}

float2 GetVisibilityUV(uint HeightfieldIndex, float2 LocalPosition)
{
	return LocalPosition * GetHeightfieldSizeInv(HeightfieldIndex);
}

float SampleHeightfield(uint HeightfieldIndex, float4 HeightfieldScaleBias, float2 LocalPosition)
{
	float4 MinMaxHeightfieldUV;
	float2 HeightfieldUV = GetHeightfieldUV(HeightfieldIndex, LocalPosition, MinMaxHeightfieldUV);
	HeightfieldUV = clamp(HeightfieldUV, MinMaxHeightfieldUV.xy, MinMaxHeightfieldUV.zw);
	return DecodePackedHeight(Texture2DSampleLevel(HeightfieldTexture, HeightfieldSampler, HeightfieldUV + 0.5f * HeightfieldScaleBias.xy, 0).xy);
}

FDFVector3 GetHeightfieldWorldPositionAndNormal(uint HeightfieldIndex, float2 LocalPosition, float2 HeightfieldUV, float CentralDiffOffset, out float3 WorldNormal, out float Visibility)
{
	//@todo - don't call GetHeightfieldScaleBias twice
	bool bHasSubsections;
	float4 HeightfieldScaleBias = GetHeightfieldScaleBias(HeightfieldIndex, bHasSubsections);

	float4 SampleValue = Texture2DSampleLevel(HeightfieldTexture, HeightfieldSampler, HeightfieldUV + .5f * HeightfieldScaleBias.xy, 0);
	float HeightfieldHeight = DecodePackedHeight(SampleValue.xy);

	float2 SampleNormal = float2(SampleValue.b, SampleValue.a) * float2(2.0,2.0) - float2(1.0,1.0);
	float3 HeightfieldNormal = float3(SampleNormal, sqrt(max(1.0 - dot(SampleNormal, SampleNormal), 0.0)));

	// Reconstruct normal using central differences
	// This is usually a better approximation for normal over the CentralDiffOffset area than SampleNormal
	{
		float2 TexelOffset = CentralDiffOffset * GetHeightfieldInvMaxScale(HeightfieldIndex);

		float R = SampleHeightfield(HeightfieldIndex, HeightfieldScaleBias, LocalPosition.xy + float2(+TexelOffset.x, 0.0f));
		float L = SampleHeightfield(HeightfieldIndex, HeightfieldScaleBias, LocalPosition.xy + float2(-TexelOffset.x, 0.0f));
		float T = SampleHeightfield(HeightfieldIndex, HeightfieldScaleBias, LocalPosition.xy + float2(0.0f, +TexelOffset.y));
		float B = SampleHeightfield(HeightfieldIndex, HeightfieldScaleBias, LocalPosition.xy + float2(0.0f, -TexelOffset.y));

		float3 HeightfieldGradient = float3((R - L) / (2.0f * TexelOffset.x), (T - B) / (2.0f * TexelOffset.y), 1.0f);
		float HeightfieldGradientLength = length(HeightfieldGradient);
		HeightfieldNormal = HeightfieldGradientLength > 0.001f ? HeightfieldGradient / HeightfieldGradientLength : HeightfieldNormal;
	}

	WorldNormal = normalize(mul(HeightfieldNormal, GetWorldToLocalRotT(HeightfieldIndex)));

	float2 VisibilityUV = GetVisibilityUV(HeightfieldIndex, LocalPosition);
	float4 SampleVisibility = Texture2DSampleLevel(VisibilityTexture, VisibilitySampler, VisibilityUV, 0);
	Visibility = 1.f - dot(SampleVisibility, GetVisibilityMask(HeightfieldIndex));

	float3 HeightfieldLocalPosition = float3(LocalPosition, HeightfieldHeight);
	FDFVector3 HeightfieldWorldPosition = DFMultiplyAffine(HeightfieldLocalPosition, GetLocalToWorld(HeightfieldIndex));

	return HeightfieldWorldPosition;
}