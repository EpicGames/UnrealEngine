// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapUtils.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"

#ifndef AVSM_BILINEAR_INTERPOLATION
#define AVSM_BILINEAR_INTERPOLATION 1
#endif //

#define INVALID_SAMPLE_INDEX 0xFFu
#define INVALID_SAMPLE_INDEX4 0xFFFFFFFFu

#if 0
// Reference single-channel implementation
struct FAVSM_Sampler
{
	FAdaptiveVolumetricShadowMap ShadowMap;
	uint PixelOffset;
	uint SampleCount;
	uint SampleIndex;

	// Query cache
	FAVSMSampleData SampleData[2];

	// Convenience creation data
	int Face;
	int2 Pixel;
	float DepthOffset;
};

bool IsValid(FAVSM_Sampler Iterator)
{
	return Iterator.SampleIndex != INVALID_SAMPLE_INDEX;
}

FAVSM_Sampler AVSM_Sampler_Create(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float DepthOffset
)
{
	FAVSM_Sampler Iterator;
	Iterator.ShadowMap = ShadowMap;

	uint LinearIndex = AVSM_LinearIndex(Iterator.ShadowMap, Face, Pixel);
	FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(Iterator.ShadowMap.IndirectionBuffer[LinearIndex]);
	Iterator.PixelOffset = IndirectionData.PixelOffset;
	Iterator.SampleCount = IndirectionData.SampleCount;

	if (Iterator.SampleCount > 0)
	{
		int TopLevelSampleCount = CalcTopLevelSampleCount(Iterator.SampleCount);
		Iterator.SampleIndex = 0;
		Iterator.SampleData[0] = AVSM_UnpackSampleData(Iterator.ShadowMap.SampleBuffer[Iterator.PixelOffset + Align4(TopLevelSampleCount) + Iterator.SampleIndex], IndirectionData);
		Iterator.SampleData[1] = AVSM_UnpackSampleData(Iterator.ShadowMap.SampleBuffer[Iterator.PixelOffset + Align4(TopLevelSampleCount) + Iterator.SampleIndex + 1], IndirectionData);
	}
	else
	{
		Iterator.SampleIndex = INVALID_SAMPLE_INDEX;
		Iterator.SampleData[0] = AVSM_CreateSampleData(0.0, 1.0);
		Iterator.SampleData[1] = AVSM_CreateSampleData(0.0, 1.0);
	}

	Iterator.Face = Face;
	Iterator.Pixel = Pixel;
	Iterator.DepthOffset = DepthOffset;
	return Iterator;
}

float AVSM_Sampler_Eval(inout FAVSM_Sampler Iterator, float X)
{
	X -= Iterator.DepthOffset;
	if (IsValid(Iterator))
	{
		int TopLevelSampleCount = CalcTopLevelSampleCount(Iterator.SampleCount);

		while ((X > Iterator.SampleData[1].X) && (Iterator.SampleIndex < Iterator.SampleCount - 1))
		{
			Iterator.SampleIndex++;
			Iterator.SampleData[0] = Iterator.SampleData[1];
			Iterator.SampleData[1] = AVSM_UnpackSampleData(Iterator.ShadowMap.SampleBuffer[Iterator.PixelOffset + Align4(TopLevelSampleCount) + Iterator.SampleIndex], Iterator.IndirectionData);
		}
	}

	float BlendWeight = saturate((X - Iterator.SampleData[0].X) * SafeRcp(Iterator.SampleData[1].X - Iterator.SampleData[0].X));
	float Tau = lerp(Iterator.SampleData[0].Tau, Iterator.SampleData[1].Tau, BlendWeight);
	return Tau;
}

FAVSMSampleData AVSM_Sampler_EvalInverse(inout FAVSM_Sampler Iterator, float Tau)
{
	if (IsValid(Iterator))
	{
		int TopLevelSampleCount = CalcTopLevelSampleCount(Iterator.SampleCount);

		while ((Tau < Iterator.SampleData[1].Tau) && (Iterator.SampleIndex < Iterator.SampleCount - 1))
		{
			Iterator.SampleIndex++;
			Iterator.SampleData[0] = Iterator.SampleData[1];
			Iterator.SampleData[1] = AVSM_UnpackSampleData(Iterator.ShadowMap.SampleBuffer[Iterator.PixelOffset + Align4(TopLevelSampleCount) + Iterator.SampleIndex], Iterator.IndirectionData);
		}
	}

	float BlendWeight = saturate((Tau - Iterator.SampleData[0].Tau) * SafeRcp(Iterator.SampleData[1].Tau - Iterator.SampleData[0].Tau));
	float X = lerp(Iterator.SampleData[0].X, Iterator.SampleData[1].X, BlendWeight);
	X += Iterator.DepthOffset;
	return AVSM_CreateSampleData(X, Tau);
}

float AVSM_Sampler_Last(inout FAVSM_Sampler Iterator)
{
	float Transmittance = 1.0;
	if (IsValid(Iterator))
	{
		int TopLevelSampleCount = CalcTopLevelSampleCount(Iterator.SampleCount);
		if (TopLevelSampleCount > 0)
		{
			uint SampleIndex = max(Iterator.SampleCount - 2, 0);
			FAVSMSampleData SampleData = AVSM_UnpackSampleData(Iterator.ShadowMap.SampleBuffer[Iterator.PixelOffset + Align4(TopLevelSampleCount) + SampleIndex], Iterator.IndirectionData);

			Transmittance = SampleData.Tau;
		}
	}
	return Transmittance;
}
#endif

// Sampler4 implementation reduces VGPR usage
struct FAVSM_Sampler4
{
	FAdaptiveVolumetricShadowMap ShadowMap;
	uint4 PixelOffset4;
	uint SampleCount4;
	float4 BeginX4;
	float4 EndX4;
	uint SampleIndex4;

	// Query cache
	uint4 SampleData0Packed4;
	uint4 SampleData1Packed4;

	// Convenience creation data
	int Face;
	float2 Pixel;
	float DepthOffset;
};

uint AVSM_Sampler4_GetSampleIndex(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	uint SampleIndex = (Sampler4.SampleIndex4 >> (Index * 8u)) & INVALID_SAMPLE_INDEX;
	return SampleIndex;
}

void AVSM_Sampler4_SetSampleIndex(inout FAVSM_Sampler4 Sampler4, int4 SampleIndex4)
{
	int SampleIndexPacked = 0;
	for (uint Index = 0; Index < 4; ++Index)
	{
		SampleIndexPacked |= ((SampleIndex4[Index] & INVALID_SAMPLE_INDEX) << (Index * 8u));
	}
	Sampler4.SampleIndex4 = SampleIndexPacked;
}

void AVSM_Sampler4_SetSampleIndex(inout FAVSM_Sampler4 Sampler4, uint Index, uint SampleIndex)
{
	uint WriteMask = INVALID_SAMPLE_INDEX << (Index * 8u);
	Sampler4.SampleIndex4 &= ~WriteMask;
	Sampler4.SampleIndex4 |= (SampleIndex & INVALID_SAMPLE_INDEX) << (Index * 8u);
}

uint AVSM_Sampler4_GetSampleCount(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	uint SampleCount = (Sampler4.SampleCount4 >> (Index * 8u)) & INVALID_SAMPLE_INDEX;
	return SampleCount;
}

float AVSM_Sampler4_GetBeginX(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	float BeginX = Sampler4.BeginX4[Index];
	return BeginX;
}

float AVSM_Sampler4_GetEndX(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	float EndX = Sampler4.EndX4[Index];
	return EndX;
}

bool AVSM_Sampler4_IsValid(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	return AVSM_Sampler4_GetSampleIndex(Sampler4, Index) != INVALID_SAMPLE_INDEX;
}

void AVSM_Sampler4_SetSampleCount(inout FAVSM_Sampler4 Sampler4, int4 SampleCount4)
{
	uint SampleCountPacked = 0;
	for (uint Index = 0; Index < 4; ++Index)
	{
		SampleCountPacked |= ((SampleCount4[Index] & INVALID_SAMPLE_INDEX) << (Index * 8u));
	}
	Sampler4.SampleCount4 = SampleCountPacked;
}

uint AVSM_Sampler4_GetPixelOffset(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	uint PixelOffset = Sampler4.PixelOffset4[Index];
	return PixelOffset;
}

void AVSM_Sampler4_SetPixelOffset(inout FAVSM_Sampler4 Sampler4, uint4 PixelOffset)
{
	Sampler4.PixelOffset4 = PixelOffset;
}

void AVSM_Sampler4_SetBeginX(inout FAVSM_Sampler4 Sampler4, float4 BeginX4)
{
	Sampler4.BeginX4 = BeginX4;
}

void AVSM_Sampler4_SetEndX(inout FAVSM_Sampler4 Sampler4, float4 EndX4)
{
	Sampler4.EndX4 = EndX4;
}

FAVSMSampleData AVSM_Sampler4_GetSampleData0(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
		AVSM_Sampler4_GetPixelOffset(Sampler4, Index),
		AVSM_Sampler4_GetSampleIndex(Sampler4, Index),
		AVSM_Sampler4_GetBeginX(Sampler4, Index),
		AVSM_Sampler4_GetEndX(Sampler4, Index)
	);

	FAVSMSampleData SampleData0 = AVSM_UnpackSampleData(Sampler4.SampleData0Packed4[Index], IndirectionData);
	SampleData0.X += Sampler4.DepthOffset;
	return SampleData0;
}

void AVSM_Sampler4_SetSampleData0(inout FAVSM_Sampler4 Sampler4, uint4 SampleDataPacked4)
{
	Sampler4.SampleData0Packed4 = SampleDataPacked4;
}

void AVSM_Sampler4_SetSampleData0(inout FAVSM_Sampler4 Sampler4, uint Index, uint SampleDataPacked)
{
	Sampler4.SampleData0Packed4[Index] = SampleDataPacked;
}

FAVSMSampleData AVSM_Sampler4_GetSampleData1(inout FAVSM_Sampler4 Sampler4, uint Index)
{
	FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
		AVSM_Sampler4_GetPixelOffset(Sampler4, Index),
		AVSM_Sampler4_GetSampleIndex(Sampler4, Index),
		AVSM_Sampler4_GetBeginX(Sampler4, Index),
		AVSM_Sampler4_GetEndX(Sampler4, Index)
	);

	FAVSMSampleData SampleData1 = AVSM_UnpackSampleData(Sampler4.SampleData1Packed4[Index], IndirectionData);
	SampleData1.X += Sampler4.DepthOffset;
	return SampleData1;
}

void AVSM_Sampler4_SetSampleData1(inout FAVSM_Sampler4 Sampler4, uint4 SampleDataPacked4)
{
	Sampler4.SampleData1Packed4 = SampleDataPacked4;
}

void AVSM_Sampler4_SetSampleData1(inout FAVSM_Sampler4 Sampler4, uint Index, uint SampleDataPacked)
{
	Sampler4.SampleData1Packed4[Index] = SampleDataPacked;
}

FAVSM_Sampler4 AVSM_Sampler4_Create(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	float2 Pixel,
	float DepthOffset
)
{
	FAVSM_Sampler4 Sampler4;
	Sampler4.ShadowMap = ShadowMap;

	uint2 Pixel4[4] = {
		clamp(Pixel, 0, ShadowMap.Resolution - 1),
		clamp(Pixel + int2(1, 0), 0, ShadowMap.Resolution - 1),
		clamp(Pixel + int2(0, 1), 0, ShadowMap.Resolution - 1),
		clamp(Pixel + int2(1, 1), 0, ShadowMap.Resolution - 1)
	};
	uint4 LinearIndex4 = uint4(
		AVSM_LinearIndex(Sampler4.ShadowMap, Face, Pixel4[0]),
		AVSM_LinearIndex(Sampler4.ShadowMap, Face, Pixel4[1]),
		AVSM_LinearIndex(Sampler4.ShadowMap, Face, Pixel4[2]),
		AVSM_LinearIndex(Sampler4.ShadowMap, Face, Pixel4[3])
	);

	uint4 SampleCount4 = INVALID_SAMPLE_INDEX4;
	uint4 SampleIndex4 = INVALID_SAMPLE_INDEX4;
	uint4 SampleData0Packed4 = 0;
	uint4 SampleData1Packed4 = 0;
	uint4 PixelOffset4 = 0;
	float4 BeginX4 = 0;
	float4 EndX4 = 0;
	for (int Index = 0; Index < 4; ++Index)
	{
		FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(ShadowMap.IndirectionBuffer[LinearIndex4[Index]]);
		PixelOffset4[Index] = IndirectionData.PixelOffset;
		SampleCount4[Index] = IndirectionData.SampleCount;
		BeginX4[Index] = IndirectionData.BeginX;
		EndX4[Index] = IndirectionData.EndX;

		if (SampleCount4[Index] > 0)
		{
			int TopLevelSampleCount = CalcTopLevelSampleCount(SampleCount4[Index]);
			SampleIndex4[Index] = 0;
			SampleData0Packed4[Index] = ShadowMap.SampleBuffer[PixelOffset4[Index] + Align4(TopLevelSampleCount) + SampleIndex4[Index]];
			SampleData1Packed4[Index] = ShadowMap.SampleBuffer[PixelOffset4[Index] + Align4(TopLevelSampleCount) + SampleIndex4[Index] + 1];
		}
		else
		{
			SampleIndex4[Index] = INVALID_SAMPLE_INDEX;
			SampleData0Packed4[Index] = AVSM_PackSampleData(0.0, 1.0, IndirectionData);
			SampleData1Packed4[Index] = AVSM_PackSampleData(0.0, 1.0, IndirectionData);
		}
	}
	AVSM_Sampler4_SetSampleCount(Sampler4, SampleCount4);
	AVSM_Sampler4_SetSampleIndex(Sampler4, SampleIndex4);
	AVSM_Sampler4_SetSampleData0(Sampler4, SampleData0Packed4);
	AVSM_Sampler4_SetSampleData1(Sampler4, SampleData1Packed4);
	AVSM_Sampler4_SetPixelOffset(Sampler4, PixelOffset4);
	AVSM_Sampler4_SetBeginX(Sampler4, BeginX4);
	AVSM_Sampler4_SetEndX(Sampler4, EndX4);

	Sampler4.Face = Face;
	Sampler4.Pixel = Pixel;
	Sampler4.DepthOffset = DepthOffset;
	return Sampler4;
}

float AVSM_Sampler4_Eval_Internal(inout FAVSM_Sampler4 Sampler4, int Index, float X)
{
	X -= Sampler4.DepthOffset;

	FAVSMSampleData SampleData[] = {
		AVSM_Sampler4_GetSampleData0(Sampler4, Index),
		AVSM_Sampler4_GetSampleData1(Sampler4, Index)
	};

	if (AVSM_Sampler4_IsValid(Sampler4, Index))
	{
		int SampleIndex = AVSM_Sampler4_GetSampleIndex(Sampler4, Index);
		int SampleCount = AVSM_Sampler4_GetSampleCount(Sampler4, Index);
		int TopLevelSampleCount = CalcTopLevelSampleCount(SampleCount);

		while ((X > SampleData[1].X) && (SampleIndex < SampleCount - 1))
		{
			SampleIndex++;
			SampleData[0] = SampleData[1];
			uint PixelOffset = AVSM_Sampler4_GetPixelOffset(Sampler4, Index);
			FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
				AVSM_Sampler4_GetPixelOffset(Sampler4, Index),
				AVSM_Sampler4_GetSampleIndex(Sampler4, Index),
				AVSM_Sampler4_GetBeginX(Sampler4, Index),
				AVSM_Sampler4_GetEndX(Sampler4, Index)
			);
			SampleData[1] = AVSM_UnpackSampleData(Sampler4.ShadowMap.SampleBuffer[PixelOffset + Align4(TopLevelSampleCount) + SampleIndex], IndirectionData);

			AVSM_Sampler4_SetSampleIndex(Sampler4, Index, SampleIndex);
			AVSM_Sampler4_SetSampleData0(Sampler4, Index, AVSM_PackSampleData(SampleData[0], IndirectionData));
			AVSM_Sampler4_SetSampleData1(Sampler4, Index, AVSM_PackSampleData(SampleData[1], IndirectionData));
		}
	}

	float BlendWeight = saturate((X - SampleData[0].X) * SafeRcp(SampleData[1].X - SampleData[0].X));
	float Tau = lerp(SampleData[0].Tau, SampleData[1].Tau, BlendWeight);
	return Tau;
}

float AVSM_Bilinear_Interpolate(float Value[4], float2 Weight)
{
	float ValueX[] = {
		lerp(Value[0], Value[1], Weight.x),
		lerp(Value[2], Value[3], Weight.x)
	};
	return lerp(ValueX[0], ValueX[1], Weight.y);
}

float AVSM_Sampler4_Eval(inout FAVSM_Sampler4 Sampler4, float X)
{
	float Tau = AVSM_Sampler4_Eval_Internal(Sampler4, 0, X);
#if AVSM_BILINEAR_INTERPOLATION != 0
	float Tau4[] = {
			Tau,
			AVSM_Sampler4_Eval_Internal(Sampler4, 1, X),
			AVSM_Sampler4_Eval_Internal(Sampler4, 2, X),
			AVSM_Sampler4_Eval_Internal(Sampler4, 3, X)
	};
	Tau = AVSM_Bilinear_Interpolate(Tau4, frac(Sampler4.Pixel));
#endif // AVSM_BILINEAR_INTERPOLATION != 0
	return Tau;
}

float AVSM_Sampler4_Eval_NoInterpolation(inout FAVSM_Sampler4 Sampler4, float X)
{
	float Tau = AVSM_Sampler4_Eval_Internal(Sampler4, 0, X);
	return Tau;
}

FAVSMSampleData AVSM_Sampler4_EvalInverse_Internal(inout FAVSM_Sampler4 Sampler4, int Index, float Tau)
{
	FAVSMSampleData SampleData[] = {
		AVSM_Sampler4_GetSampleData0(Sampler4, Index),
		AVSM_Sampler4_GetSampleData1(Sampler4, Index)
	};

	if (AVSM_Sampler4_IsValid(Sampler4, Index))
	{
		int SampleCount = AVSM_Sampler4_GetSampleCount(Sampler4, Index);
		int TopLevelSampleCount = CalcTopLevelSampleCount(SampleCount);

		int SampleIndex = AVSM_Sampler4_GetSampleIndex(Sampler4, Index);
		uint PixelOffset = AVSM_Sampler4_GetPixelOffset(Sampler4, Index);
		while ((Tau < SampleData[1].Tau) && (SampleIndex < SampleCount - 1))
		{
			SampleIndex++;
			SampleData[0] = SampleData[1];
			FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
				AVSM_Sampler4_GetPixelOffset(Sampler4, Index),
				AVSM_Sampler4_GetSampleIndex(Sampler4, Index),
				AVSM_Sampler4_GetBeginX(Sampler4, Index),
				AVSM_Sampler4_GetEndX(Sampler4, Index)
			);
			SampleData[1] = AVSM_UnpackSampleData(Sampler4.ShadowMap.SampleBuffer[PixelOffset + Align4(TopLevelSampleCount) + SampleIndex], IndirectionData);

			AVSM_Sampler4_SetSampleData0(Sampler4, Index, AVSM_PackSampleData(SampleData[0], IndirectionData));
			AVSM_Sampler4_SetSampleData1(Sampler4, Index, AVSM_PackSampleData(SampleData[1], IndirectionData));
		}
	}

	float BlendWeight = saturate((Tau - SampleData[0].Tau) * SafeRcp(SampleData[1].Tau - SampleData[0].Tau));
	float X = lerp(SampleData[0].X, SampleData[1].X, BlendWeight);
	X += Sampler4.DepthOffset;
	return AVSM_CreateSampleData(X, Tau);
}

FAVSMSampleData AVSM_Sampler4_EvalInverse(inout FAVSM_Sampler4 Sampler4, float Tau)
{
	FAVSMSampleData SampleData = AVSM_Sampler4_EvalInverse_Internal(Sampler4, 0, Tau);
#if AVSM_BILINEAR_INTERPOLATION != 0
	float X4[] = {
		SampleData.X,
		AVSM_Sampler4_EvalInverse_Internal(Sampler4, 1, Tau).X,
		AVSM_Sampler4_EvalInverse_Internal(Sampler4, 2, Tau).X,
		AVSM_Sampler4_EvalInverse_Internal(Sampler4, 3, Tau).X
	};
	SampleData.X = AVSM_Bilinear_Interpolate(X4, frac(Sampler4.Pixel));
#endif // AVSM_BILINEAR_INTERPOLATION != 0
	return SampleData;
}

FAVSMSampleData AVSM_Sampler4_Last_Internal(inout FAVSM_Sampler4 Sampler4, int Index)
{
	FAVSMSampleData SampleData = AVSM_CreateSampleData(0.0, 1.0);

	if (AVSM_Sampler4_IsValid(Sampler4, Index))
	{
		int SampleCount = AVSM_Sampler4_GetSampleCount(Sampler4, Index);
		int TopLevelSampleCount = CalcTopLevelSampleCount(SampleCount);
		if (TopLevelSampleCount > 0)
		{
			uint SampleIndex = max(SampleCount - 2, 0);
			uint PixelOffset = AVSM_Sampler4_GetPixelOffset(Sampler4, Index);
			FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
				AVSM_Sampler4_GetPixelOffset(Sampler4, Index),
				AVSM_Sampler4_GetSampleIndex(Sampler4, Index),
				AVSM_Sampler4_GetBeginX(Sampler4, Index),
				AVSM_Sampler4_GetEndX(Sampler4, Index)
			);
			SampleData = AVSM_UnpackSampleData(Sampler4.ShadowMap.SampleBuffer[PixelOffset + Align4(TopLevelSampleCount) + SampleIndex], IndirectionData);
			SampleData.X += Sampler4.DepthOffset;
		}
	}

	return SampleData;
}

FAVSMSampleData AVSM_Sampler4_Last(inout FAVSM_Sampler4 Sampler4)
{
	FAVSMSampleData SampleData = AVSM_Sampler4_Last_Internal(Sampler4, 0);
#if AVSM_BILINEAR_INTERPOLATION != 0
	// TODO: Filter X when sentinel value is removed
	//float Depth4[] = {
	//	SampleData.X,
	//	AVSM_Sampler4_Last_Internal(Sampler4, 1).X,
	//	AVSM_Sampler4_Last_Internal(Sampler4, 2).X,
	//	AVSM_Sampler4_Last_Internal(Sampler4, 3).X
	//};
	//SampleData.X = AVSM_Bilinear_Interpolate(Depth4, frac(Sampler4.Pixel));

	float Tau4[] = {
		SampleData.Tau,
		AVSM_Sampler4_Last_Internal(Sampler4, 1).Tau,
		AVSM_Sampler4_Last_Internal(Sampler4, 2).Tau,
		AVSM_Sampler4_Last_Internal(Sampler4, 3).Tau
	};
	SampleData.Tau = AVSM_Bilinear_Interpolate(Tau4, frac(Sampler4.Pixel));
#endif // AVSM_BILINEAR_INTERPOLATION != 0
	return SampleData;
}
