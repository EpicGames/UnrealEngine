// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapUtils.ush"
#include "HeterogeneousVolumesBeerShadowMapUtils.ush"

#define AVSM_SAMPLE_MODE_DISABLED 0
#define AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED 1
#define AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED_LOOP 2
//#define AVSM_SAMPLE_MODE_TWO_LEVEL_REFERENCE 3
//#define AVSM_SAMPLE_MODE_BINARY_SEARCH 4
//#define AVSM_SAMPLE_MODE_REFERENCE 5

#ifndef AVSM_SAMPLE_MODE
#define AVSM_SAMPLE_MODE AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED_LOOP
#endif // AVSM_SAMPLE_MODE

// Adaptive Volumetric Shadow Map implementation

struct FAdaptiveVolumetricShadowMap
{
	float4x4 TranslatedWorldToShadow[6];
	float3 TranslatedWorldOrigin[6];
	float4 TranslatedWorldPlane[6];
	float4 SplitDepths[6];
	float DownsampleFactor;

	int2 Resolution;
	int NumShadowMatrices;
	int MaxSampleCount;
	bool bIsEmpty;
	bool bIsDirectionalLight;

	StructuredBuffer<uint4> IndirectionBuffer;
	StructuredBuffer<uint> SampleBuffer;
	Texture2D<float4> RadianceTexture;
	SamplerState TextureSampler;
};

bool AVSM_PixelAndDepth(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	float3 TranslatedWorldPosition,
	inout float2 Pixel,
	inout float Depth
)
{
	Depth = 0.0;
	Pixel = -1;
	float4 ShadowPositionAsFloat4 = mul(float4(TranslatedWorldPosition, 1), ShadowMap.TranslatedWorldToShadow[Face]);
	if (ShadowPositionAsFloat4.w > 0)
	{
		float3 ShadowPosition = ShadowPositionAsFloat4.xyz / ShadowPositionAsFloat4.w;
		if (all(ShadowPosition.xyz > 0) || all(ShadowPosition.xy < 1))
		{
			Pixel = clamp(ShadowPosition.xy * ShadowMap.Resolution - 0.5, 0, ShadowMap.Resolution - 1);
			if (ShadowMap.bIsDirectionalLight)
			{
				Depth = dot(ShadowMap.TranslatedWorldPlane[Face].xyz, TranslatedWorldPosition) + ShadowMap.TranslatedWorldPlane[Face].w;
			}
			return true;
		}
	}

	return false;
}

uint AVSM_LinearIndex(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	uint Face,
	uint2 PixelCoord
)
{
	uint LinearIndex = Face * ShadowMap.Resolution.x * ShadowMap.Resolution.y + PixelCoord.y * ShadowMap.Resolution.x + PixelCoord.x;
	return LinearIndex;
}

float SafeRcp(float Value)
{
	return Value > 0 ? rcp(Value) : 0.0;
}

float SafeLog(float Value)
{
	return Value > 0 ? log(Value) : 0.0;
}

#define INTERPOLANT_TYPE_LINEAR 0
#define INTERPOLANT_TYPE_EXPONENTIAL 1
#define INTERPOLANT_TYPE_EXPONENTIAL_FIT 2
#define INTERPOLANT_TYPE INTERPOLANT_TYPE_EXPONENTIAL_FIT

float Interpolate(
	FAVSMSampleData ShadowData[2],
	float WorldHitT
)
{
	float Transmittance = ShadowData[0].Tau;

	if (INTERPOLANT_TYPE == INTERPOLANT_TYPE_LINEAR)
	{
		// Linear interpolant
		float DeltaX = max(ShadowData[1].X - ShadowData[0].X, 0.0);
		float X = clamp(WorldHitT - ShadowData[0].X, 0.0, DeltaX);
		float BlendWeight = saturate(X * SafeRcp(DeltaX));
		Transmittance = lerp(ShadowData[0].Tau, ShadowData[1].Tau, BlendWeight);
	}
	else if (INTERPOLANT_TYPE == INTERPOLANT_TYPE_EXPONENTIAL)
	{
		// Exponential interpolant
		float DeltaX = max(ShadowData[1].X - ShadowData[0].X, 0.0);
		float X = clamp(WorldHitT - ShadowData[0].X, 0.0, DeltaX);
		float BlendWeight = saturate(X * SafeRcp(DeltaX));
		//float SigmaT = 10.0;
		float SigmaT = 1.0;
		Transmittance = lerp(ShadowData[1].Tau, ShadowData[0].Tau, exp(-BlendWeight * SigmaT));
	}
	else // if (INTERPOLANT_TYPE == INTERPOLANT_TYPE_EXPONENTIAL_FIT)
	{
		// Exponential fit
		float DeltaX = max(ShadowData[1].X - ShadowData[0].X, 0.0);
		float SigmaT = -log(ShadowData[1].Tau * SafeRcp(ShadowData[0].Tau)) * SafeRcp(DeltaX);
		float X = clamp(WorldHitT - ShadowData[0].X, 0.0, DeltaX);
		Transmittance = saturate(ShadowData[0].Tau * exp(-X * SigmaT));
	}

	return Transmittance;
}

float AVSM_Sample_Reference(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float WorldHitT
)
{
	uint LinearIndex = AVSM_LinearIndex(ShadowMap, Face, Pixel);
	FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(ShadowMap.IndirectionBuffer[LinearIndex]);
	if (WorldHitT < IndirectionData.BeginX)
	{
		return 1.0;
	}

	FAVSMSampleData ShadowData[2] = {
		AVSM_CreateSampleData(0.0, 1.0),
		AVSM_CreateSampleData(0.0, 1.0)
	};

	int TopLevelSampleCount = CalcTopLevelSampleCount(IndirectionData.SampleCount);
	if (TopLevelSampleCount > 0)
	{
		// Early-out
		if (WorldHitT >= IndirectionData.EndX)
		{
			int LastSampleIndex = IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndirectionData.SampleCount - 1;
			return AVSM_UnpackSampleData(ShadowMap.SampleBuffer[LastSampleIndex], IndirectionData).Tau;
		}

		// Initialize from the top-level
		ShadowData[0] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset], IndirectionData);

		// Iterate through the linear span of offset transmittance information
		for (int ElementIndex = 0; ElementIndex < IndirectionData.SampleCount - 1; ++ElementIndex)
		{
			ShadowData[1] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + ElementIndex], IndirectionData);
			if (WorldHitT < ShadowData[1].X)
			{
				break;
			}

			ShadowData[0] = ShadowData[1];
		}
	}

	float Transmittance = Interpolate(ShadowData, WorldHitT);
	return Transmittance;
}

float AVSM_Sample_BinarySearch(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float WorldHitT
)
{
	uint LinearIndex = AVSM_LinearIndex(ShadowMap, Face, Pixel);
	FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(ShadowMap.IndirectionBuffer[LinearIndex]);
	if (WorldHitT < IndirectionData.BeginX)
	{
		return 1.0;
	}

	FAVSMSampleData ShadowData[2] = {
		AVSM_CreateSampleData(0.0, 1.0),
		AVSM_CreateSampleData(0.0, 1.0)
	};

	int TopLevelSampleCount = CalcTopLevelSampleCount(IndirectionData.SampleCount);
	if (TopLevelSampleCount > 0)
	{
		// Early-out
		if (WorldHitT >= IndirectionData.EndX)
		{
			int LastSampleIndex = IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndirectionData.SampleCount - 1;
			return AVSM_UnpackSampleData(ShadowMap.SampleBuffer[LastSampleIndex], IndirectionData).Tau;
		}

		// Initialize from the top-level
		ShadowData[0] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset], IndirectionData);

		// Account for the shifted bottom-level
		int ElementIndexMin = -1;
		int ElementIndexMax = IndirectionData.SampleCount - 1;
		while (ElementIndexMax - ElementIndexMin > 1)
		{
			int ElementIndex = (ElementIndexMin + ElementIndexMax + 1) / 2;
			FAVSMSampleData SampleData = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + ElementIndex], IndirectionData);

			if (WorldHitT < SampleData.X)
			{
				ElementIndexMax = ElementIndex;
				ShadowData[1] = SampleData;
			}
			else
			{
				ElementIndexMin = ElementIndex;
				ShadowData[0] = SampleData;
			}
		}
	}

	float Transmittance = Interpolate(ShadowData, WorldHitT);
	return Transmittance;
}
#if 0
float AVSM_Sample_TwoLevel_Reference(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float WorldHitT
)
{
	uint LinearIndex = AVSM_LinearIndex(ShadowMap, Face, Pixel);
	FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(ShadowMap.IndirectionBuffer[LinearIndex]);
	if (WorldHitT < IndirectionData.BeginX)
	{
		return 1.0;
	}

	FAVSMSampleData ShadowData[2] = {
		AVSM_CreateSampleData(0.0, 1.0),
		AVSM_CreateSampleData(0.0, 1.0)
	};

	int TopLevelSampleIndex = 0;

	// Level 1
	int TopLevelSampleCount = CalcTopLevelSampleCount(IndirectionData.SampleCount);
	if (TopLevelSampleCount > 0)
	{
		if (WorldHitT >= IndirectionData.EndX)
		{
			int LastSampleIndex = IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndirectionData.SampleCount - 1;
			return AVSM_UnpackSampleData(AVSM.SampleBuffer[LastSampleIndex], IndirectionData).Tau;
		}

		ShadowData[0] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset], IndirectionData);

		// Iterate through the top-level
		for (TopLevelSampleIndex = 0; TopLevelSampleIndex < TopLevelSampleCount; ++TopLevelSampleIndex)
		{
			ShadowData[1] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset + TopLevelSampleIndex], IndirectionData);
			if (WorldHitT < ShadowData[1].X)
			{
				break;
			}

			ShadowData[0] = ShadowData[1];
		}
	}

	// Level 2
	int SampleIndexMin = max((TopLevelSampleIndex - 1) * 4, 0);
	// Account for the index shifting in the bottom-level span
	int SampleIndexMax = min(TopLevelSampleIndex * 4, IndirectionData.SampleCount - 1);
	for (int SampleIndex = SampleIndexMin; SampleIndex < SampleIndexMax; ++SampleIndex)
	{
		ShadowData[1] = AVSM_UnpackSampleData(ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + SampleIndex], IndirectionData);
		if (WorldHitT < ShadowData[1].X)
		{
			break;
		}

		ShadowData[0] = ShadowData[1];
	}

	// Lerp the final value
	float BlendWeight = saturate((WorldHitT - ShadowData[0].X) * SafeRcp(ShadowData[1].X - ShadowData[0].X));
	float Transmittance = lerp(ShadowData[0].Tau, ShadowData[1].Tau, BlendWeight);
	return Transmittance;
}
#endif

float AVSM_Sample_TwoLevel_Vectorized(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float WorldHitT
)
{
	uint LinearIndex = AVSM_LinearIndex(ShadowMap, Face, Pixel);
	FAVSMIndirectionData IndirectionData = AVSM_UnpackIndirectionData(ShadowMap.IndirectionBuffer[LinearIndex]);
	if (WorldHitT < IndirectionData.BeginX)
	{
		return 1.0;
	}

	FAVSMSampleData ShadowData[2] = {
		AVSM_CreateSampleData(0.0, 1.0),
		AVSM_CreateSampleData(0.0, 1.0)
	};

	int TopLevelSampleIndex = 0;

	// Level 1
	int TopLevelSampleCount = CalcTopLevelSampleCount(IndirectionData.SampleCount);
	if (TopLevelSampleCount > 0)
	{
		if (WorldHitT >= IndirectionData.EndX)
		{
			int LastIndex = IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndirectionData.SampleCount - 1;
			return AVSM_UnpackSampleData(ShadowMap.SampleBuffer[LastIndex], IndirectionData).Tau;
		}

		int TopLevelSampleOffset = 0;
#if AVSM_SAMPLE_MODE == AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED_LOOP
		for (; TopLevelSampleOffset < TopLevelSampleCount; TopLevelSampleOffset += 4)
#endif
		{
			int4 IndexOffset = TopLevelSampleOffset + int4(0, 1, 2, 3);

			int4 PackedShadowData4 = int4(
				ShadowMap.SampleBuffer[IndirectionData.PixelOffset + IndexOffset[0]],
				ShadowMap.SampleBuffer[IndirectionData.PixelOffset + IndexOffset[1]],
				ShadowMap.SampleBuffer[IndirectionData.PixelOffset + IndexOffset[2]],
				ShadowMap.SampleBuffer[IndirectionData.PixelOffset + IndexOffset[3]]
			);

			float4 Depths = float4(
				AVSM_UnpackSampleData(PackedShadowData4[0], IndirectionData).X,
				AVSM_UnpackSampleData(PackedShadowData4[1], IndirectionData).X,
				AVSM_UnpackSampleData(PackedShadowData4[2], IndirectionData).X,
				AVSM_UnpackSampleData(PackedShadowData4[3], IndirectionData).X
			);

			int4 Result = float4(WorldHitT, WorldHitT, WorldHitT, WorldHitT) > Depths;
			int LocalSampleIndex = Result[0] + Result[1] + Result[2] + Result[3];

			ShadowData[0] = AVSM_UnpackSampleData(PackedShadowData4[max(LocalSampleIndex - 1, 0)], IndirectionData);
			ShadowData[1] = AVSM_UnpackSampleData(PackedShadowData4[min(LocalSampleIndex, 3)], IndirectionData);
			TopLevelSampleIndex = min(TopLevelSampleOffset + LocalSampleIndex, TopLevelSampleCount);
#if AVSM_SAMPLE_MODE == AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED_LOOP
			if (LocalSampleIndex < 4)
			{
				break;
			}
#endif
		}
	}

	// Level 2
	if (TopLevelSampleCount > 0)
	{
		int4 IndexOffset = max((TopLevelSampleIndex - 1) * 4, 0);
		IndexOffset += int4(0, 1, 2, 3);

		int4 PackedShadowData4 = int4(
			ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndexOffset.x],
			ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndexOffset.y],
			ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndexOffset.z],
			ShadowMap.SampleBuffer[IndirectionData.PixelOffset + Align4(TopLevelSampleCount) + IndexOffset.w]
		);

		float4 Depths = float4(
			AVSM_UnpackSampleData(PackedShadowData4.x, IndirectionData).X,
			AVSM_UnpackSampleData(PackedShadowData4.y, IndirectionData).X,
			AVSM_UnpackSampleData(PackedShadowData4.z, IndirectionData).X,
			AVSM_UnpackSampleData(PackedShadowData4.w, IndirectionData).X
		);

		int4 Result = float4(WorldHitT, WorldHitT, WorldHitT, WorldHitT) > Depths;
		int Index = Result.x + Result.y + Result.z + Result.w;

		if (Index > 0)
		{
			ShadowData[0] = AVSM_UnpackSampleData(PackedShadowData4[max(Index - 1, 0)], IndirectionData);
		}
		ShadowData[1] = AVSM_UnpackSampleData(PackedShadowData4[min(Index, 3)], IndirectionData);
	}

	float Transmittance = Interpolate(ShadowData, WorldHitT);
	return Transmittance;
}

float AVSM_Sample(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	int Face,
	int2 Pixel,
	float WorldHitT
)
{
	return
// Temporarily disabling vectorized loop for reference
#if AVSM_SAMPLE_MODE == AVSM_SAMPLE_MODE_TWO_LEVEL_VECTORIZED_LOOP
		AVSM_Sample_BinarySearch(
#else
		AVSM_Sample_TwoLevel_Vectorized(
#endif
			ShadowMap,
			Face,
			Pixel,
			WorldHitT
		);
}

float AVSM_SampleTransmittance(
	inout FAdaptiveVolumetricShadowMap ShadowMap,
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	if (any(ShadowMap.Resolution == 0))
	{
		return 1.0;
	}

	float Transmittance[] = { 1.0, 1.0 };
	float Alpha = 1.0;

	float Depth = length(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);

	int Face[] = {
		ShadowMap.NumShadowMatrices - 1,
		ShadowMap.NumShadowMatrices - 1
	};

	if (AVSM.bIsDirectionalLight)
	{
		// Find the highest quality cascade for the shade point
		for (; Face[1] > 0; --Face[1])
		{
			float Far = ShadowMap.SplitDepths[Face[1]].y;
			if (Depth <= Far)
			{
				break;
			}
		}
		Face[0] = max(Face[1] - 1, 0);

		// Apply near-far distance fades
		float Near = ShadowMap.SplitDepths[Face[1]].x;
		float NearFadeDist = ShadowMap.SplitDepths[Face[1]].z;
		if (Depth - Near < NearFadeDist)
		{
			Alpha = saturate((Depth - Near) / NearFadeDist);
			Face[0] = min(Face[1] + 1, ShadowMap.NumShadowMatrices - 1);
		}

		float Far = ShadowMap.SplitDepths[Face[1]].y;
		float FarFadeDist = ShadowMap.SplitDepths[Face[1]].w;
		if (Far - Depth < FarFadeDist)
		{
			Alpha = saturate((Far - Depth) / FarFadeDist);
			Face[0] = max(Face[1] - 1, 0);
		}
	}
	else if (AVSM.NumShadowMatrices > 1)
	{
		float3 LightDir = TranslatedWorldPosition - ShadowMap.TranslatedWorldOrigin[0];
		float3 LightDirAbs = abs(LightDir);

		Face[0] = LightDirAbs[1] > LightDirAbs[0] ? 1 : 0;
		Face[0] = LightDirAbs[2] > LightDirAbs[Face[0]] ? 2 : Face[0];

		int FaceOffset = (sign(LightDir[Face[0]]) > 0) ? 1 : 0;
		Face[0] = Face[1] = Face[0] * 2 + FaceOffset;
	}

	float4 ShadowPositionAsFloat4 = 0;
	float3 ShadowPosition = 0;
	for (int Index = 0; Index < 2; ++Index)
	{
		ShadowPositionAsFloat4 = mul(float4(TranslatedWorldPosition, 1), ShadowMap.TranslatedWorldToShadow[Face[Index]]);
		if (ShadowPositionAsFloat4.w > 0)
		{
			ShadowPosition = ShadowPositionAsFloat4.xyz / ShadowPositionAsFloat4.w;
			if (all(ShadowPosition.xyz > 0) || all(ShadowPosition.xy < 1))
			{
				float2 ShadowMapCoord = clamp(ShadowPosition.xy * ShadowMap.Resolution - 0.5, 0, ShadowMap.Resolution - 1);
				float ShadowZ = length(TranslatedWorldPosition - ShadowMap.TranslatedWorldOrigin[Face[Index]]);
				if (ShadowMap.bIsDirectionalLight)
				{
					ShadowZ = dot(ShadowMap.TranslatedWorldPlane[Face[Index]].xyz, TranslatedWorldPosition) + ShadowMap.TranslatedWorldPlane[Face[Index]].w;
				}

#define BILINEAR_INTERPOLATION 1
#if BILINEAR_INTERPOLATION
				// Bilinear interpolation of shadow data
				int2 ShadowMapCoordX[4] =
				{
					clamp(ShadowMapCoord, 0, ShadowMap.Resolution - 1),
					clamp(ShadowMapCoord + int2(1, 0), 0, ShadowMap.Resolution - 1),
					clamp(ShadowMapCoord + int2(0, 1), 0, ShadowMap.Resolution - 1),
					clamp(ShadowMapCoord + int2(1, 1), 0, ShadowMap.Resolution - 1)
				};

				float TransmittanceX[4] =
				{
					AVSM_Sample(ShadowMap, Face[Index], ShadowMapCoordX[0], ShadowZ),
					AVSM_Sample(ShadowMap, Face[Index], ShadowMapCoordX[1], ShadowZ),
					AVSM_Sample(ShadowMap, Face[Index], ShadowMapCoordX[2], ShadowZ),
					AVSM_Sample(ShadowMap, Face[Index], ShadowMapCoordX[3], ShadowZ)
				};

				float2 Weight = frac(ShadowMapCoord);
				float TransmittanceY0 = lerp(TransmittanceX[0], TransmittanceX[1], Weight.x);
				float TransmittanceY1 = lerp(TransmittanceX[2], TransmittanceX[3], Weight.x);
				Transmittance[Index] = lerp(TransmittanceY0, TransmittanceY1, Weight.y);
#else
				Transmittance[Index] = AVSM_Sample(ShadowMap, Face[Index], ShadowMapCoord, ShadowZ);
#endif
			}
		}
	}

	return ((1.0 - Alpha) * Transmittance[0]) + (Alpha * Transmittance[1]);
}

FAdaptiveVolumetricShadowMap GetAdaptiveVolumetricShadowMap()
{
	FAdaptiveVolumetricShadowMap ShadowMap;
	for (int i = 0; i < 6; ++i)
	{
		ShadowMap.TranslatedWorldToShadow[i] = AVSM.TranslatedWorldToShadow[i];
		ShadowMap.TranslatedWorldOrigin[i] = AVSM.TranslatedWorldOrigin[i].xyz;
		ShadowMap.TranslatedWorldPlane[i] = AVSM.TranslatedWorldPlane[i];
		ShadowMap.SplitDepths[i] = AVSM.SplitDepths[i];
	}

	ShadowMap.Resolution = AVSM.Resolution;
	ShadowMap.DownsampleFactor = AVSM.DownsampleFactor;
	ShadowMap.NumShadowMatrices = AVSM.NumShadowMatrices;
	ShadowMap.MaxSampleCount = AVSM.MaxSampleCount;
	ShadowMap.bIsEmpty = AVSM.bIsEmpty;
	ShadowMap.bIsDirectionalLight = AVSM.bIsDirectionalLight;

	ShadowMap.IndirectionBuffer = AVSM.IndirectionBuffer;
	ShadowMap.SampleBuffer = AVSM.SampleBuffer;

	ShadowMap.RadianceTexture = AVSM.RadianceTexture;
	ShadowMap.TextureSampler = AVSM.TextureSampler;
	return ShadowMap;
}

FAdaptiveVolumetricShadowMap GetAdaptiveVolumetricShadowMapForCamera()
{
	FAdaptiveVolumetricShadowMap ShadowMap;
#if USE_CAMERA_AVSM
	for (int i = 0; i < 6; ++i)
	{
		ShadowMap.TranslatedWorldToShadow[i] = CameraAVSM.TranslatedWorldToShadow[i];
		ShadowMap.TranslatedWorldOrigin[i] = CameraAVSM.TranslatedWorldOrigin[i].xyz;
		ShadowMap.TranslatedWorldPlane[i] = CameraAVSM.TranslatedWorldPlane[i];
		ShadowMap.SplitDepths[i] = CameraAVSM.SplitDepths[i];
	}

	ShadowMap.Resolution = CameraAVSM.Resolution;
	ShadowMap.DownsampleFactor = CameraAVSM.DownsampleFactor;
	ShadowMap.NumShadowMatrices = CameraAVSM.NumShadowMatrices;
	ShadowMap.MaxSampleCount = CameraAVSM.MaxSampleCount;
	ShadowMap.bIsEmpty = CameraAVSM.bIsEmpty;
	ShadowMap.bIsDirectionalLight = CameraAVSM.bIsDirectionalLight;

	ShadowMap.IndirectionBuffer = CameraAVSM.IndirectionBuffer;
	ShadowMap.SampleBuffer = CameraAVSM.SampleBuffer;

	ShadowMap.RadianceTexture = CameraAVSM.RadianceTexture;
	ShadowMap.TextureSampler = CameraAVSM.TextureSampler;
#endif // USE_CAMERA_AVSM
	return ShadowMap;
}

float AVSM_SampleTransmittance(
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	FAdaptiveVolumetricShadowMap AdaptiveVolumetricShadowMap = GetAdaptiveVolumetricShadowMap();
	return AVSM_SampleTransmittance(AdaptiveVolumetricShadowMap, TranslatedWorldPosition, LightTranslatedWorldPosition);
}

float4 AVSM_SampleCameraRadianceAndTransmittance4(
	float2 ScreenUV,
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	FAdaptiveVolumetricShadowMap AdaptiveVolumetricShadowMap = GetAdaptiveVolumetricShadowMap();
	float Transmittance = AVSM_SampleTransmittance(AdaptiveVolumetricShadowMap, TranslatedWorldPosition, LightTranslatedWorldPosition);

	// Extract radiance from precomposited render target and weight based on relative opacity
	float4 Result = Texture2DSample(AdaptiveVolumetricShadowMap.RadianceTexture, AdaptiveVolumetricShadowMap.TextureSampler, ScreenUV);
	float3 Radiance = Result.rgb * View.OneOverPreExposure;
	float Weight = saturate((1.0 - Transmittance) * SafeRcp(1.0 - Result.a));

	return float4(Radiance * Weight, Transmittance);
}

// Beer Shadow Map implementation

struct FBeerShadowMap
{
	float4x4 TranslatedWorldToShadow[6];
	float3 TranslatedWorldOrigin[6];
	float4 TranslatedWorldPlane[6];
	float4 SplitDepths[6];
	float DownsampleFactor;

	int2 Resolution;
	int NumShadowMatrices;
	int MaxSampleCount;
	bool bIsEmpty;
	bool bIsDirectionalLight;

	Texture2D<float4> BeerShadowMapTexture;
	Texture2D<float4> RadianceTexture;
	SamplerState TextureSampler;
};

FBeerShadowMap GetBeerShadowMap()
{
	FBeerShadowMap ShadowMap;
	for (int i = 0; i < 6; ++i)
	{
		ShadowMap.TranslatedWorldToShadow[i] = BeerShadowMap.TranslatedWorldToShadow[i];
		ShadowMap.TranslatedWorldOrigin[i] = BeerShadowMap.TranslatedWorldOrigin[i].xyz;
		ShadowMap.TranslatedWorldPlane[i] = BeerShadowMap.TranslatedWorldPlane[i];
		ShadowMap.SplitDepths[i] = BeerShadowMap.SplitDepths[i];
	}

	ShadowMap.Resolution = BeerShadowMap.Resolution;
	ShadowMap.DownsampleFactor = BeerShadowMap.DownsampleFactor;
	ShadowMap.NumShadowMatrices = BeerShadowMap.NumShadowMatrices;
	ShadowMap.MaxSampleCount = BeerShadowMap.MaxSampleCount;
	ShadowMap.bIsEmpty = BeerShadowMap.bIsEmpty;
	ShadowMap.bIsDirectionalLight = BeerShadowMap.bIsDirectionalLight;

	ShadowMap.BeerShadowMapTexture = BeerShadowMap.BeerShadowMapTexture;
	ShadowMap.RadianceTexture = BeerShadowMap.RadianceTexture;
	ShadowMap.TextureSampler = BeerShadowMap.TextureSampler;
	return ShadowMap;
}

float BeerShadowMap_Sample(
	inout FBeerShadowMap ShadowMap,
	float2 UV,
	float ShadowZ
)
{
	float Transmittance = 1.0;

	// NOTE: Consider using point-sampler for depth and bilinear-sampler for transmittance to improve quality
	float4 PackedData = Texture2DSample(ShadowMap.BeerShadowMapTexture, ShadowMap.TextureSampler, UV);
	FBeerShadowMapData ShadowMapData = UnpackBeerShadowMapData(PackedData);
	if (ShadowZ > ShadowMapData.T[0])
	{
		FAVSMSampleData SampleData[] = {
			AVSM_CreateSampleData(ShadowMapData.T[0], ShadowMapData.Tau[0]),
			AVSM_CreateSampleData(ShadowMapData.T[1], ShadowMapData.Tau[1])
		};
		Transmittance = Interpolate(SampleData, ShadowZ);
	}

	return Transmittance;
}

float BeerShadowMap_SampleTransmittanceSimple(
	inout FBeerShadowMap ShadowMap,
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	if (any(ShadowMap.Resolution == 0))
	{
		return 1.0;
	}

	float Transmittance[] = { 1.0, 1.0 };
	float Alpha = 0.0;

	float Depth = length(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);

	int Face[] = {
		ShadowMap.NumShadowMatrices - 1,
		ShadowMap.NumShadowMatrices - 1
	};

	float4 ShadowPositionAsFloat4 = 0;
	float3 ShadowPosition = 0;
	for (int Index = 0; Index < 1; ++Index)
	{
		ShadowPositionAsFloat4 = mul(float4(TranslatedWorldPosition, 1), ShadowMap.TranslatedWorldToShadow[Face[Index]]);
		if (ShadowPositionAsFloat4.w > 0)
		{
			ShadowPosition = ShadowPositionAsFloat4.xyz / ShadowPositionAsFloat4.w;

			// Remap ShadowPosition to Texture UV
			ShadowPosition.x = (Face[Index] + ShadowPosition.x) / BeerShadowMap.NumShadowMatrices;
			float3 ClipUV[] = {
				float3(float(Face[Index]) / BeerShadowMap.NumShadowMatrices, 0, -1),
				float3(float(Face[Index] + 1) / BeerShadowMap.NumShadowMatrices, 1, 1)
			};

			if (all(ShadowPosition > ClipUV[0]) && all(ShadowPosition < ClipUV[1]))
			{
				float ShadowZ = length(TranslatedWorldPosition - ShadowMap.TranslatedWorldOrigin[Face[Index]]);
				if (ShadowMap.bIsDirectionalLight)
				{
					ShadowZ = dot(ShadowMap.TranslatedWorldPlane[Face[Index]].xyz, TranslatedWorldPosition) + ShadowMap.TranslatedWorldPlane[Face[Index]].w;
				}

#define BILINEAR_INTERPOLATION 1
#if BILINEAR_INTERPOLATION
				// Bilinear interpolation of shadow data
				float2 ShadowMapCoord = ShadowPosition.xy * ShadowMap.Resolution;
				int2 ShadowMapCoordX[4] =
				{
					clamp(ShadowMapCoord, 0, ShadowMap.Resolution),
					clamp(ShadowMapCoord + int2(1, 0), 0, ShadowMap.Resolution),
					clamp(ShadowMapCoord + int2(0, 1), 0, ShadowMap.Resolution),
					clamp(ShadowMapCoord + int2(1, 1), 0, ShadowMap.Resolution)
				};

				float TransmittanceX[4] =
				{
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[0]) / ShadowMap.Resolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[1]) / ShadowMap.Resolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[2]) / ShadowMap.Resolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[3]) / ShadowMap.Resolution, ShadowZ)
				};

				float2 Weight = frac(ShadowMapCoord);
				float TransmittanceY0 = lerp(TransmittanceX[0], TransmittanceX[1], Weight.x);
				float TransmittanceY1 = lerp(TransmittanceX[2], TransmittanceX[3], Weight.x);
				Transmittance[Index] = lerp(TransmittanceY0, TransmittanceY1, Weight.y);
#else
				Transmittance[Index] = BeerShadowMap_Sample(ShadowMap, ShadowPosition.xy, ShadowZ);
#endif
			}
		}
	}

	return ((1.0 - Alpha) * Transmittance[0]) + (Alpha * Transmittance[1]);
}

float BeerShadowMap_SampleTransmittance(
	inout FBeerShadowMap ShadowMap,
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	if (any(ShadowMap.Resolution == 0))
	{
		return 1.0;
	}

	float Transmittance[] = { 1.0, 1.0 };
	float Alpha = 1.0;

	float Depth = length(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);

	int Face[] = {
		ShadowMap.NumShadowMatrices - 1,
		ShadowMap.NumShadowMatrices - 1
	};

	if (BeerShadowMap.bIsDirectionalLight)
	{
		// Find the highest quality cascade for the shade point
		for (; Face[1] > 0; --Face[1])
		{
			float Far = ShadowMap.SplitDepths[Face[1]].y;
			if (Depth <= Far)
			{
				break;
			}
		}
		Face[0] = max(Face[1] - 1, 0);

		// Apply near-far distance fades
		float Near = ShadowMap.SplitDepths[Face[1]].x;
		float NearFadeDist = ShadowMap.SplitDepths[Face[1]].z;
		if (Depth - Near < NearFadeDist)
		{
			Alpha = saturate((Depth - Near) / NearFadeDist);
			Face[0] = min(Face[1] + 1, ShadowMap.NumShadowMatrices - 1);
		}

		float Far = ShadowMap.SplitDepths[Face[1]].y;
		float FarFadeDist = ShadowMap.SplitDepths[Face[1]].w;
		if (Far - Depth < FarFadeDist)
		{
			Alpha = saturate((Far - Depth) / FarFadeDist);
			Face[0] = max(Face[1] - 1, 0);
		}
	}
	else if (BeerShadowMap.NumShadowMatrices > 1)
	{
		float3 LightDir = TranslatedWorldPosition - ShadowMap.TranslatedWorldOrigin[0];
		float3 LightDirAbs = abs(LightDir);

		Face[0] = LightDirAbs[1] > LightDirAbs[0] ? 1 : 0;
		Face[0] = LightDirAbs[2] > LightDirAbs[Face[0]] ? 2 : Face[0];

		int FaceOffset = (sign(LightDir[Face[0]]) > 0) ? 1 : 0;
		Face[0] = Face[1] = Face[0] * 2 + FaceOffset;
	}

	float4 ShadowPositionAsFloat4 = 0;
	float3 ShadowPosition = 0;
	for (int Index = 0; Index < 2; ++Index)
	{
		ShadowPositionAsFloat4 = mul(float4(TranslatedWorldPosition, 1), ShadowMap.TranslatedWorldToShadow[Face[Index]]);
		if (ShadowPositionAsFloat4.w > 0)
		{
			ShadowPosition = ShadowPositionAsFloat4.xyz / ShadowPositionAsFloat4.w;

			// Remap ShadowPosition to Texture UV
			ShadowPosition.x = (Face[Index] + ShadowPosition.x) / BeerShadowMap.NumShadowMatrices;
			float3 ClipUV[] = {
				float3(float(Face[Index]) / BeerShadowMap.NumShadowMatrices, 0, -1),
				float3(float(Face[Index] + 1) / BeerShadowMap.NumShadowMatrices, 1, 1)
			};

			if (all(ShadowPosition > ClipUV[0]) && all(ShadowPosition < ClipUV[1]))
			{
				float ShadowZ = length(TranslatedWorldPosition - ShadowMap.TranslatedWorldOrigin[Face[Index]]);
				if (ShadowMap.bIsDirectionalLight)
				{
					ShadowZ = dot(ShadowMap.TranslatedWorldPlane[Face[Index]].xyz, TranslatedWorldPosition) + ShadowMap.TranslatedWorldPlane[Face[Index]].w;
				}

#define BILINEAR_INTERPOLATION 1
#if BILINEAR_INTERPOLATION
				int2 AtlasResolution = ShadowMap.Resolution * int2(BeerShadowMap.NumShadowMatrices, 1);

				// Bilinear interpolation of shadow data
				float2 ShadowMapCoord = ShadowPosition.xy * AtlasResolution;
				int2 ShadowMapCoordX[4] =
				{
					clamp(ShadowMapCoord, 0, AtlasResolution),
					clamp(ShadowMapCoord + int2(1, 0), 0, AtlasResolution),
					clamp(ShadowMapCoord + int2(0, 1), 0, AtlasResolution),
					clamp(ShadowMapCoord + int2(1, 1), 0, AtlasResolution)
				};

				float TransmittanceX[4] =
				{
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[0]) / AtlasResolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[1]) / AtlasResolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[2]) / AtlasResolution, ShadowZ),
					BeerShadowMap_Sample(ShadowMap, float2(ShadowMapCoordX[3]) / AtlasResolution, ShadowZ)
				};

				float2 Weight = frac(ShadowMapCoord);
				float TransmittanceY0 = lerp(TransmittanceX[0], TransmittanceX[1], Weight.x);
				float TransmittanceY1 = lerp(TransmittanceX[2], TransmittanceX[3], Weight.x);
				Transmittance[Index] = lerp(TransmittanceY0, TransmittanceY1, Weight.y);
#else
				Transmittance[Index] = BeerShadowMap_Sample(ShadowMap, ShadowPosition.xy, ShadowZ);
#endif
			}
		}
	}

	return ((1.0 - Alpha) * Transmittance[0]) + (Alpha * Transmittance[1]);
}

float BeerShadowMap_SampleTransmittance(
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	FBeerShadowMap BeerShadowMap = GetBeerShadowMap();
	return BeerShadowMap_SampleTransmittance(BeerShadowMap, TranslatedWorldPosition, LightTranslatedWorldPosition);
}

float4 BeerShadowMap_SampleCameraRadianceAndTransmittance4(
	float2 ScreenUV,
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
	FBeerShadowMap BeerShadowMap = GetBeerShadowMap();
	float Transmittance = BeerShadowMap_SampleTransmittanceSimple(BeerShadowMap, TranslatedWorldPosition, LightTranslatedWorldPosition);

	// Extract radiance from precomposited render target and weight based on relative opacity
	float4 Result = Texture2DSample(BeerShadowMap.RadianceTexture, BeerShadowMap.TextureSampler, ScreenUV);
	float3 Radiance = Result.rgb * View.OneOverPreExposure;
	float Weight = saturate((1.0 - Transmittance) * SafeRcp(1.0 - Result.a));

	return float4(Radiance * Weight, Transmittance);
}
