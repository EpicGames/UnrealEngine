// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define AVSM_NULL_PTR 0xFF

struct FAVSMSampleData
{
	float X;
	float Tau;
};

#define AVSM_INVALID_PIXEL_OFFSET 0x3FFFFFF

struct FAVSMIndirectionData
{
	uint PixelOffset;
	int SampleCount;
	float BeginX;
	float EndX;
};

/* AVSM Sample Data API */

FAVSMSampleData AVSM_CreateSampleData(float X, float Tau)
{
	FAVSMSampleData SampleData;
	SampleData.X = X;
	SampleData.Tau = Tau;
	return SampleData;
}

FAVSMSampleData AVSM_CreateSampleData(float X, float3 Transmittance)
{
	return AVSM_CreateSampleData(X, Luminance(Transmittance));
}

uint AVSM_PackSampleDataX(float X, FAVSMIndirectionData IndirectionData)
{
	float Extent = IndirectionData.EndX - IndirectionData.BeginX;
	float NormalizedX = (X - IndirectionData.BeginX) / Extent;
	uint FixedX = saturate(NormalizedX) * MaxHalfFloat;
	return FixedX;
}

float AVSM_UnpackSampleDataX(uint FixedX, FAVSMIndirectionData IndirectionData)
{
	float Extent = IndirectionData.EndX - IndirectionData.BeginX;
	float NormalizedX = FixedX / MaxHalfFloat;
	float X = saturate(NormalizedX) * Extent + IndirectionData.BeginX;
	return X;
}

int AVSM_PackSampleData(FAVSMSampleData SampleData, FAVSMIndirectionData IndirectionData)
{
	int PackedData = 0;
	PackedData |= AVSM_PackSampleDataX(SampleData.X, IndirectionData);
	PackedData |= f32tof16(SampleData.Tau) << 16;
	return PackedData;
}

FAVSMSampleData AVSM_UnpackSampleData(int PackedData, FAVSMIndirectionData IndirectionData)
{
	FAVSMSampleData SampleData;
	SampleData.X = AVSM_UnpackSampleDataX(PackedData & 0xFFFF, IndirectionData);
	SampleData.Tau = f16tof32(PackedData >> 16);
	return SampleData;
}

int AVSM_PackSampleData(float X, float Tau, FAVSMIndirectionData IndirectionData)
{
	return AVSM_PackSampleData(AVSM_CreateSampleData(X, Tau), IndirectionData);
}

/* AVSM Linked List API */

FAVSMSampleData AVSM_GetLinkedListSampleData(uint2 PackedData)
{
	FAVSMSampleData SampleData;
	SampleData.X = asfloat(PackedData.x);
	SampleData.Tau = f16tof32(PackedData.y >> 16);
	return SampleData;
}

void AVSM_SetLinkedListSampleData(inout uint2 PackedData, FAVSMSampleData SampleData)
{
	PackedData.x = asuint(SampleData.X);
	PackedData.y = (PackedData.y & 0xFFFF) |
		(f32tof16(SampleData.Tau) << 16);
}

uint AVSM_GetLinkedListPrevPtr(uint2 PackedData)
{
	return (PackedData.y >> 8) & AVSM_NULL_PTR;
}

uint AVSM_GetLinkedListNextPtr(uint2 PackedData)
{
	return PackedData.y & AVSM_NULL_PTR;
}

void AVSM_SetLinkedListPrevPtr(inout uint2 PackedData, uint PrevPtr)
{
	PackedData.y = (PackedData.y & 0xFFFF00FF) |
		((PrevPtr & AVSM_NULL_PTR) << 8);
}

void AVSM_SetLinkedListNextPtr(inout uint2 PackedData, uint NextPtr)
{
	PackedData.y = (PackedData.y & 0xFFFFFF00) |
		(NextPtr & AVSM_NULL_PTR);
}

/* AVSM Indirection Data API */

FAVSMIndirectionData AVSM_CreateIndirectionData(uint PixelOffset, int SampleCount, float BeginX, float EndX)
{
	FAVSMIndirectionData IndirectionData;
	IndirectionData.PixelOffset = PixelOffset;
	IndirectionData.SampleCount = SampleCount;
	IndirectionData.BeginX = BeginX;
	IndirectionData.EndX = EndX;
	return IndirectionData;
}

uint4 AVSM_PackIndirectionData(FAVSMIndirectionData IndirectionData)
{
	uint4 PackedData = 0;
	PackedData.x = ((IndirectionData.PixelOffset & AVSM_INVALID_PIXEL_OFFSET) << 6) | (IndirectionData.SampleCount & 0x3f);
	PackedData.y = asuint(IndirectionData.BeginX);
	PackedData.z = asuint(IndirectionData.EndX);
	return PackedData;
}

FAVSMIndirectionData AVSM_UnpackIndirectionData(uint4 PackedData)
{
	FAVSMIndirectionData IndirectionData;
	IndirectionData.PixelOffset = (PackedData.x >> 6) & AVSM_INVALID_PIXEL_OFFSET;
	IndirectionData.SampleCount = PackedData.x & 0x3f;
	IndirectionData.BeginX = asfloat(PackedData.y);
	IndirectionData.EndX = asfloat(PackedData.z);
	return IndirectionData;
}

int Align4(int Value)
{
	return (Value + 3) & ~3;
}

int CalcTopLevelSampleCount(int SampleCount)
{
	int TopLevelSampleCount = Align4(SampleCount) >> 2;
	return TopLevelSampleCount;
}