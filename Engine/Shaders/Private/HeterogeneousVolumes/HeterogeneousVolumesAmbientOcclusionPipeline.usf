// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS 0

#include "/Engine/Generated/Material.ush"
#include "HeterogeneousVolumesStochasticFiltering.ush"
#include "HeterogeneousVolumesLiveShadingUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesLightingUtils.ush"
#include "../BlueNoise.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

#ifndef USE_EXISTENCE_MASK
#define USE_EXISTENCE_MASK 1
#endif //

// Object data
float4x4 LocalToWorld;
float4x4 WorldToLocal;
float3 LocalBoundsOrigin;
float3 LocalBoundsExtent;
int PrimitiveId;

// Volume data
int3 VoxelResolution;
int3 VoxelMin;
int3 VoxelMax;

// Shadowing
float ShadowStepSize;
float ShadowStepFactor;

// Ray data
float MaxTraceDistance;
float MaxShadowTraceDistance;
float StepSize;
float StepFactor;
int MaxStepCount;
int bJitter;
int StochasticFilteringMode;

Texture3D ExistenceMaskTexture;

// Optional cinematic features
int bIsOfflineRender;

// AO data
int2 NumRays;

FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

float4x4 GetLocalToWorld()
{
	return LocalToWorld;
}

float4x4 GetWorldToLocal()
{
	return WorldToLocal;
}

float3 GetLocalBoundsOrigin()
{
	return LocalBoundsOrigin;
}

float3 GetLocalBoundsExtent()
{
	return LocalBoundsExtent;
}

int3 GetVolumeResolution()
{
	return VoxelResolution;
}

float GetStepSize()
{
	return StepSize;
}

float GetShadowStepSize()
{
	return ShadowStepSize;
}

float GetStepFactor()
{
	return StepFactor;
}

float GetShadowStepFactor()
{
	return ShadowStepFactor;
}

float GetMaxTraceDistance()
{
	return MaxTraceDistance;
}

float GetMaxShadowTraceDistance()
{
	return MaxShadowTraceDistance;
}

int GetStochasticFilteringMode()
{
	return StochasticFilteringMode;
}

float EvalAmbientOcclusion(float3 WorldPosition)
{
	return 0.0;
}

float GetIndirectInscatteringFactor()
{
	return 0.0;
}

// Material sampling hooks..
struct FVolumeSampleContext
{
	FMaterialPixelParameters MaterialParameters;
	FPixelMaterialInputs PixelMaterialInputs;
};

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 WorldPosition_DDX, float3 WorldPosition_DDY, float MipLevel)
{
	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.MaterialParameters = MakeInitializedMaterialPixelParameters();
	//VolumeSampleContext.MaterialParameters.PrimitiveId = PrimitiveId;
	VolumeSampleContext.MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
	VolumeSampleContext.MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
	// TODO: Add object centroid to LWC.ObjectWorldPosition
	VolumeSampleContext.MaterialParameters.LWCData.LocalToWorld = WSPromote(GetLocalToWorld());
	VolumeSampleContext.MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(GetWorldToLocal());

#if USE_ANALYTIC_DERIVATIVES
	VolumeSampleContext.MaterialParameters.WorldPosition_DDX = WorldPosition_DDX;
	VolumeSampleContext.MaterialParameters.WorldPosition_DDY = WorldPosition_DDY;
	CalcPixelMaterialInputsAnalyticDerivatives(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#else
	CalcPixelMaterialInputs(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#endif

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0.0;
	float3 WorldPosition_DDY = 0.0;
	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int StochasticFilteringMode)
{
	// TODO: Use FilterWidth determine MipLevel
	//float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (StochasticFilteringMode == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;
	return CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	float3 Extinction = SampleExtinctionCoefficients(Context.PixelMaterialInputs);
	return Extinction;
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	float3 Emission = SampleEmissive(Context.PixelMaterialInputs);
	return Emission;
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = SampleAlbedo(Context.PixelMaterialInputs);
	return Albedo;
}

// Cinematic feature options

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
	return INDIRECT_LIGHTING_MODE_OFF;
}

bool IsOfflineRender()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return bIsOfflineRender;
#else
	return false;
#endif
}

int ShouldApplyHeightFog()
{
	return false;
}

int ShouldApplyVolumetricFog()
{
	return false;
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
	return FOG_INSCATTERING_MODE_OFF;
}

#include "HeterogeneousVolumesRayMarchingUtils.ush"

// Output
RWTexture3D<uint> RWAmbientOcclusionUIntTexture;

FRayMarchingContext CreateShadowRayMarchingContext(
	float3 WorldRayOrigin,
	float3 WorldToLight,
	uint MaxStepCount
)
{
	float3 WorldRayEnd = WorldRayOrigin + WorldToLight;
	float3 WorldRayDirection = normalize(WorldToLight);

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayTMin = 0.0;
	float LocalRayTMax = length(LocalRayDirection);
	LocalRayDirection /= LocalRayTMax;

	float ShadowBias = 0.5;
	float ShadowStepSize = CalcShadowStepSize(LocalRayDirection);
	int bApplyEmissionAndTransmittance = 0;
	int bApplyDirectLighting = 0;
	int bApplyShadowTransmittance = 0;

	FRayMarchingContext ShadowRayMarchingContext = CreateRayMarchingContext(
		LocalRayOrigin,
		LocalRayDirection,
		LocalRayTMin,
		LocalRayTMax,
		WorldRayOrigin,
		WorldRayDirection,
		ShadowBias,
		ShadowStepSize,
		MaxStepCount,
		bApplyEmissionAndTransmittance,
		bApplyDirectLighting,
		bApplyShadowTransmittance
		//RayMarchingContext.MaxShadowTraceDistance
	);

	return ShadowRayMarchingContext;
}

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RenderAmbientOcclusionWithLiveShadingCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	int3 VoxelCount = VoxelMax - VoxelMin + 1;
	int3 VoxelIndex = float3(DispatchThreadId.x, DispatchThreadId.y, DispatchThreadId.z % VoxelCount.z);
	if (any(VoxelIndex >= VoxelCount))
	{
		return;
	}
	VoxelIndex += VoxelMin;

	int LinearRayIndex = DispatchThreadId.z / VoxelCount.z;
	int2 RayIndex = int2(LinearRayIndex % NumRays.x, LinearRayIndex / NumRays.x);
	if (any(RayIndex >= NumRays))
	{
		return;
	}

	float3 VoxelJitter = 0.5;
	if (bJitter)
	{
		uint3 Rand32Bits = Rand4DPCG32(uint4(VoxelIndex, View.StateFrameIndexMod8)).xyz;
		VoxelJitter = float3(Rand32Bits) / float(uint(0xffffffff));
	}
	float3 UVW = (VoxelIndex + VoxelJitter) / float3(GetLightingCacheResolution());

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float3 LocalRayOrigin = UVW * GetLocalBoundsExtent() * 2.0 + LocalBoundsMin;
	float3 WorldRayOrigin = mul(float4(LocalRayOrigin, 1.0), GetLocalToWorld()).xyz;

	// Construct ray direction
	float2 RayUV = float2(RayIndex + 0.5) / NumRays;
	float3 WorldRayDirection = EquiAreaSphericalMapping(RayUV);

	// Evaluate transmittance based on MaxShadowDistance
	FRayMarchingContext RayMarchingContext = CreateShadowRayMarchingContext(WorldRayOrigin, WorldRayDirection * MaxShadowTraceDistance, MaxStepCount);

	float AmbientOcclusion = 1.0;
#if USE_EXISTENCE_MASK
	float Existence = ExistenceMaskTexture.SampleLevel(View.SharedBilinearClampedSampler, UVW, 0).r;
	if (Existence > 0.0)
#endif
	{
		float3 Transmittance = ComputeTransmittance(RayMarchingContext);
		AmbientOcclusion = Luminance(Transmittance) * rcp(NumRays.x * NumRays.y);
	}

	// Convert to fixed-point and accumulate
	int AmbientOcclusionFixedPoint = AmbientOcclusion * 65535.0;
	InterlockedAdd(RWAmbientOcclusionUIntTexture[VoxelIndex], AmbientOcclusionFixedPoint);
}

RWTexture3D<float> RWExistenceMaskTexture;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RenderExistenceMaskWithLiveShadingCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	int3 VoxelCount = VoxelMax - VoxelMin + 1;
	int3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex > VoxelCount))
	{
		return;
	}
	VoxelIndex += VoxelMin;

	float3 VoxelJitter = 0.5;
	if (bJitter)
	{
		uint3 Rand32Bits = Rand4DPCG32(uint4(VoxelIndex, View.StateFrameIndexMod8)).xyz;
		VoxelJitter = float3(Rand32Bits) / float(uint(0xffffffff));
	}
	float3 UVW = (VoxelIndex + VoxelJitter) / float3(VoxelResolution);

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float3 LocalPosition = UVW * GetLocalBoundsExtent() * 2.0 + LocalBoundsMin;
	float3 WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;

	int MipLevel = 0;

	FVolumeSampleContext SampleContext;
	if (IsOfflineRender())
	{
		float3 FilterWidth = 1.0;
		int LinearIndex = VoxelIndex.x * (VoxelResolution.y * VoxelResolution.z) + VoxelIndex.y * VoxelResolution.z + VoxelIndex.z;
		float Rand = RandomSequenceCreate(LinearIndex, View.StateFrameIndex).Get1D();
		int StochasticFilteringMode = GetStochasticFilteringMode();
		SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, FilterWidth, MipLevel, Rand, StochasticFilteringMode);
	}
	else
	{
		SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
	}

	float3 Extinction = SampleExtinction(SampleContext);
	RWExistenceMaskTexture[VoxelIndex] = any(Extinction > 0.0) ? 1.0 : 0.0;
}
