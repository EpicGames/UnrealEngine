// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Beer Shadow Map implementation
struct FBeerShadowMapData
{
	float T[2];
	float Tau[2];
};

FBeerShadowMapData InitializeBeerShadowMapData(float Transmittance, float MaxTraceDistance)
{
	FBeerShadowMapData Data;
	Data.T[0] = MaxTraceDistance;
	Data.T[1] = 0.0;
	Data.Tau[0] = Transmittance;
	Data.Tau[1] = Transmittance;
	return Data;
}

FBeerShadowMapData UnpackBeerShadowMapData(float4 PackedData)
{
	FBeerShadowMapData Data;
	Data.T[0] = PackedData.r;
	Data.T[1] = PackedData.g;
	Data.Tau[0] = PackedData.b;
	Data.Tau[1] = PackedData.a;
	return Data;
}

float4 PackBeerShadowMapData(FBeerShadowMapData Data)
{
	float4 PackedData;
	PackedData.r = Data.T[0];
	PackedData.g = Data.T[1];
	PackedData.b = Data.Tau[0];
	PackedData.a = Data.Tau[1];
	return PackedData;
}

#define BEER_SHADOW_MAP_COMBINE_METHOD_FIRST 0
#define BEER_SHADOW_MAP_COMBINE_METHOD_EXPAND 1

FBeerShadowMapData ExpandBeerShadowMapData(FBeerShadowMapData Data0, FBeerShadowMapData Data1)
{
	FBeerShadowMapData Result;
	Result.T[0] = min(Data0.T[0], Data1.T[0]);
	Result.T[1] = max(Data0.T[1], Data1.T[1]);
	Result.Tau[0] = max(Data0.Tau[0], Data0.Tau[0]);
	Result.Tau[1] = min(Data0.Tau[1], Data1.Tau[1]);
	return Result;
}

bool IsValid(FBeerShadowMapData Data)
{
	return Data.T[1] > Data.T[0];
}

FBeerShadowMapData FirstBeerShadowMapData(FBeerShadowMapData Data0, FBeerShadowMapData Data1)
{
	bool bIsValid0 = IsValid(Data0);
	bool bIsValid1 = IsValid(Data1);

	if (bIsValid0 && bIsValid1)
	{
		if (Data0.T[0] < Data1.T[0])
		{
			return Data0;
		}
		else
		{
			return Data1;
		}
	}
	else if (bIsValid0)
	{
		return Data0;
	}
	// else
	return Data1;
}

FBeerShadowMapData CombineBeerShadowMapData(FBeerShadowMapData Data0, FBeerShadowMapData Data1, int CombineMethod)
{
	FBeerShadowMapData Result;

	if (!IsValid(Data0))
	{
		return Data1;
	}

	if (!IsValid(Data1))
	{
		return Data0;
	}

	if (CombineMethod == BEER_SHADOW_MAP_COMBINE_METHOD_FIRST)
	{
		Result = FirstBeerShadowMapData(Data0, Data1);
	}
	else // if (CombineMethod == BEER_SHADOW_MAP_COMBINE_METHOD_EXPAND)
	{
		Result = ExpandBeerShadowMapData(Data0, Data1);
	}
	return Result;
}
