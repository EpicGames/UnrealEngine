// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"

#ifndef UPSAMPLE_MODE
#define UPSAMPLE_MODE 0
#endif // UPSAMPLE_MODE

#define UPSAMPLE_MODE_NEAREST 1
#define UPSAMPLE_MODE_BILINEAR 2

#ifndef FILTER_MODE
#define FILTER_MODE 0
#endif // FILTER_MODE

#define FILTER_MODE_BILATERAL 1
#define FILTER_MODE_GAUSSIAN_3X3 2
#define FILTER_MODE_GAUSSIAN_5x5 3

Texture2D SceneDepthTexture;
Texture2D<float4> HeterogeneousVolumeRadiance;
Texture2D<float> HeterogeneousVolumeHoldout;
RWTexture2D<float4> RWColorTexture;
int DownsampleFactor;
int bUseAVSM;

float SampleTransmittance(uint2 ViewPixelCoord)
{
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(float4(ViewPixelCoord + View.TemporalAAJitter.xy, DeviceZ, 1));
	return AVSM_SampleTransmittance(TranslatedWorldPosition, ResolvedView.TranslatedWorldCameraOrigin);
}

RWTexture2D<float4> RWUpsampledTexture;
SamplerState TextureSampler;
uint2 UpsampledResolution;
uint2 UpsampledViewRect;
uint2 UpsampledViewRectMin;

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void HeterogeneousVolumesUpsampleCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	if (any(DispatchThreadId.xy >= UpsampledViewRect))
	{
		return;
	}
	int2 UpsampledPixelCoord = DispatchThreadId.xy + UpsampledViewRectMin;

#if UPSAMPLE_MODE == UPSAMPLE_MODE_BILINEAR
	float2 UV = (UpsampledPixelCoord + 0.5 * DownsampleFactor) / float2(UpsampledResolution);
	UV = clamp(UV, 0, (UpsampledViewRect + UpsampledViewRectMin - 1) / float2(UpsampledResolution));
	float4 Result = Texture2DSample(HeterogeneousVolumeRadiance, TextureSampler, UV);
#else
	int2 DownsampledPixelCoord = UpsampledPixelCoord / DownsampleFactor;
	float4 Result = HeterogeneousVolumeRadiance[DownsampledPixelCoord];
#endif

	RWUpsampledTexture[UpsampledPixelCoord] = Result;
}

int FilterWidth;
RWTexture2D<float4> RWFilteredTexture;

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void HeterogeneousVolumesFilterCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	if (any(DispatchThreadId.xy >= UpsampledViewRect))
	{
		return;
	}

	int2 PixelCoord = DispatchThreadId.xy + UpsampledViewRectMin;
	float4 Radiance = HeterogeneousVolumeRadiance[PixelCoord];
	float HeterogeneousVolumeTransmittance = Radiance.a;

	// Bilaterial Filter
#if FILTER_MODE == FILTER_MODE_BILATERAL
	{
		Radiance = 0.0;
		float WeightSum = 0.0;

		uint FilterHalfWidth = FilterWidth / 2;
		float FilterRadius2 = max(2.0 * FilterWidth * FilterWidth, 1.0);

		for (int X = 0; X < FilterWidth; ++X)
		{
			for (int Y = 0; Y < FilterWidth; ++Y)
			{
				int2 Offset = int2(X, Y) - FilterHalfWidth;
				int2 SampleCoord = PixelCoord + Offset;
				SampleCoord = clamp(SampleCoord, 0, int2(UpsampledViewRect + UpsampledViewRectMin) - 1);
				float FilterWeight = max(FilterRadius2 - dot(Offset, Offset), 0.0);

				float4 Value = HeterogeneousVolumeRadiance[SampleCoord];
				float3 RadianceDelta = HeterogeneousVolumeRadiance[PixelCoord].rgb - Value.rgb;
				float RadianceWeight = saturate(1.0 - dot(RadianceDelta, RadianceDelta));

				float TauDelta = HeterogeneousVolumeTransmittance - Value.a;
				float TauWeight = 1.0 - TauDelta * TauDelta;

				Radiance += Value * FilterWeight * RadianceWeight * TauWeight;
				WeightSum += FilterWeight * RadianceWeight * TauWeight;

			}
		}

		Radiance /= WeightSum;
	}
	// Gaussian 3x3 Filter
#elif FILTER_MODE == FILTER_MODE_GAUSSIAN_3X3
	{
		float Gaussian3x3Weights[] = {
			1.0, 2.0, 1.0,
			2.0, 4.0, 2.0,
			1.0, 2.0, 1.0
		};

		Radiance = 0.0;
		float WeightSum = 0.0;

		uint FilterHalfWidth = 1;
		for (int X = 0; X < 3; ++X)
		{
			for (int Y = 0; Y < 3; ++Y)
			{
				int2 OffsetCoord = int2(X, Y);
				int2 SampleCoord = PixelCoord + OffsetCoord - FilterHalfWidth;
				SampleCoord = clamp(SampleCoord, 0, int2(UpsampledViewRect + UpsampledViewRectMin) - 1);
				float FilterWeight = Gaussian3x3Weights[OffsetCoord.y * 3 + OffsetCoord.x];

				Radiance += HeterogeneousVolumeRadiance[SampleCoord] * FilterWeight;
				WeightSum += FilterWeight;
			}
		}

		Radiance /= WeightSum;
	}
	// Gaussian 5x5 Filter
#elif FILTER_MODE == FILTER_MODE_GAUSSIAN_5x5
	{
		float Gaussian5x5Weights[] = {
			1.0, 4.0, 7.0, 4.0, 1.0,
			4.0, 16.0, 26.0, 16.0, 4.0,
			7.0, 26.0, 41.0, 26.0, 7.0,
			4.0, 16.0, 26.0, 16.0, 4.0,
			1.0, 4.0, 7.0, 4.0, 1.0,
		};

		Radiance = 0.0;
		float WeightSum = 0.0;

		uint FilterHalfWidth = 2;
		for (int X = 0; X < 5; ++X)
		{
			for (int Y = 0; Y < 5; ++Y)
			{
				int2 OffsetCoord = int2(X, Y);
				int2 SampleCoord = PixelCoord + OffsetCoord - FilterHalfWidth;
				SampleCoord = clamp(SampleCoord, 0, int2(UpsampledViewRect + UpsampledViewRectMin) - 1);
				float FilterWeight = Gaussian5x5Weights[OffsetCoord.y * 5 + OffsetCoord.x];

				Radiance += HeterogeneousVolumeRadiance[SampleCoord] * FilterWeight;
				WeightSum += FilterWeight;
			}
		}

		Radiance /= WeightSum;
	}
#endif

	RWFilteredTexture[PixelCoord] = Radiance;
}

int2 GetDownsampledResolution(int2 Resolution, int Factor)
{
	return (Resolution + Factor - 1) / Factor;
}

int2 GetScaledViewRect()
{
	return GetDownsampledResolution(View.ViewSizeAndInvSize.xy, DownsampleFactor);
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void HeterogeneousVolumesCompositeCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	ResolvedView = ResolveView();
	if (any(DispatchThreadId >= View.ViewSizeAndInvSize.xy))
	{
		return;
	}
	uint2 ViewPixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy;
	uint2 PixelCoord = ViewPixelCoord / DownsampleFactor;

	// Cache Sample values
	float4 Radiance = HeterogeneousVolumeRadiance[PixelCoord];
	float HeterogeneousVolumeTransmittance = bUseAVSM ? SampleTransmittance(ViewPixelCoord) : Radiance.a;
	//float HeterogeneousVolumeTransmittance = Radiance.a;

	float4 Result;
	Result.rgb = Radiance.rgb + RWColorTexture[ViewPixelCoord].rgb * HeterogeneousVolumeTransmittance;
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (View.bPrimitiveAlphaHoldoutEnabled)
	{
		float SurfaceOpacity = RWColorTexture[ViewPixelCoord].a * HeterogeneousVolumeTransmittance;
		float VolumeOpacity = HeterogeneousVolumeHoldout[PixelCoord];
		Result.a = SurfaceOpacity + VolumeOpacity;
	}
#else
	Result.a = RWColorTexture[ViewPixelCoord].a * HeterogeneousVolumeTransmittance;
#endif

	RWColorTexture[ViewPixelCoord] = Result;
}
