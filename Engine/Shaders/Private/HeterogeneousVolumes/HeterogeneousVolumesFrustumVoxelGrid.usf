// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesVoxelGridTypes.ush"

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../ComputeShaderUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesVoxelGridTraversal.ush"

int3 TopLevelGridResolution;

StructuredBuffer<FTopLevelGridData> TopLevelGridBuffer;
StructuredBuffer<FScalarGridData> ExtinctionGridBuffer;
StructuredBuffer<FVectorGridData> EmissionGridBuffer;
StructuredBuffer<FVectorGridData> ScatteringGridBuffer;

float3 PrimitiveWorldBoundsMin;
float3 PrimitiveWorldBoundsMax;

RWStructuredBuffer<FTopLevelGridData> RWTopLevelGridBuffer;

float4x4 ViewToWorld;
int3 VoxelDimensions;

float TanHalfFOV;
float NearPlaneDepth;
float FarPlaneDepth;

bool BoxIntersectsVoxel(float3 WorldBoxCenter, float WorldBoxRadius, uint3 VoxelIndex)
{
	float3 VoxelPosMin = VoxelIndex;
	float3 VoxelPosMax = (VoxelIndex + 1);

	float3 ViewPosMin = VoxelToView(VoxelPosMin, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);
	float3 ViewPosMax = VoxelToView(VoxelPosMax, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

	float3 WorldPosMin = mul(float4(ViewPosMin, 1), ViewToWorld).xyz;
	float3 WorldPosMax = mul(float4(ViewPosMax, 1), ViewToWorld).xyz;

	float3 WorldPosCenter = (WorldPosMin + WorldPosMax) * 0.5;
	float WorldRadius = length((WorldPosMax - WorldPosMin) * 0.5);

	float DistanceToCenters = length(WorldBoxCenter - WorldPosCenter);
	float SumOfRadii = WorldBoxRadius + WorldRadius;
	return DistanceToCenters < SumOfRadii;
}

// Mark voxels which intersect a given primitive
[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void MarkTopLevelGridVoxelsForFrustumGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	uint LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
	FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[LinearIndex];
	uint3 VoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

	float3 WorldPrimitiveOrigin = (PrimitiveWorldBoundsMax + PrimitiveWorldBoundsMin) * 0.5;
	float3 WorldPrimitiveExtent = (PrimitiveWorldBoundsMax - PrimitiveWorldBoundsMin) * 0.5;
	if (BoxIntersectsVoxel(WorldPrimitiveOrigin, length(WorldPrimitiveExtent), VoxelIndex))
	{
		VoxelResolution = 4;
	}

	SetBottomLevelIndex(TopLevelGridData, EMPTY_VOXEL_INDEX);
	SetBottomLevelVoxelResolution(TopLevelGridData, VoxelResolution);
	RWTopLevelGridBuffer[LinearIndex] = TopLevelGridData;
}
