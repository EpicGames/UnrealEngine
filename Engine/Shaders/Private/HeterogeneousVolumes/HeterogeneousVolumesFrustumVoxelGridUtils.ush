// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float Exponential(float sigma, float x)
{
	// X = 1 - e^(-sigma * x)
	return 1 - exp(-sigma * x);
}

float InverseExponential(float sigma, float X)
{
	// X = 1 - e^(-sigma * x)
	// x = ln(1 - X) / -sigma
	return log(1 - X) / -sigma;
}

float FindSigma(float x, float X)
{
	// X = 1 - e^(-sigma * x)
	// e^(-sigma * x) = 1 - X
	// -sigma * x = ln(1 - X)
	// sigma = -ln(1 - X) / x
	return -log(1 - X) / x;
}

float ExponentialRemap(float ViewDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float Sigma = FindSigma(FarPlaneDepth, 0.9999);
	float NormalizedDepth = Exponential(Sigma, ViewDepth);
	return NormalizedDepth;
}

float InverseExponentialRemap(float NormalizedDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float Sigma = FindSigma(FarPlaneDepth, 0.9999);
	float ViewDepth = InverseExponential(Sigma, NormalizedDepth);
	return ViewDepth;
}

float LinearRemap(float ViewDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float NormalizedDepth = (ViewDepth - NearPlaneDepth) / (FarPlaneDepth - NearPlaneDepth);
	return NormalizedDepth;
}

float InverseLinearRemap(float NormalizedDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float ViewDepth = NormalizedDepth * (FarPlaneDepth - NearPlaneDepth) + NearPlaneDepth;
	return ViewDepth;
}

float SquareDistanceRemap(float ViewDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float NormalizedDepth = ViewDepth / 20.0;
	if (ViewDepth > 2.0)
	{
		float LogDepth = log2(ViewDepth);
		float LogFarPlaneDepth = log2(FarPlaneDepth);
		NormalizedDepth = LogDepth / LogFarPlaneDepth + 0.1;
	}

	return NormalizedDepth;
}

float InverseSquareDistanceRemap(float NormalizedDepth, float NearPlaneDepth, float FarPlaneDepth)
{
	float ViewDepth = NormalizedDepth * 20.0;
	if (ViewDepth > 2.0)
	{
		float LogFarPlaneDepth = log2(FarPlaneDepth);
		float LogDepth = (NormalizedDepth - 0.1) * LogFarPlaneDepth;
		ViewDepth = pow(2, LogDepth);
	}

	return ViewDepth;
}

float3 ScreenToVoxel(float3 ScreenPos, int3 VoxelDimensions)
{
	float3 NDC = float3(ScreenPos.xy * 0.5 + 0.5, ScreenPos.z);
	float3 VoxelPos = NDC * VoxelDimensions;
	return VoxelPos;
}

float3 VoxelToScreen(float3 VoxelPos, int3 VoxelDimensions)
{
	float3 NDC = VoxelPos / VoxelDimensions;
	float3 ScreenPos = float3(NDC.xy * 2.0 - 1.0, NDC.z);
	return ScreenPos;
}

float PerspectiveRemap(float ViewDepth, int3 VoxelDimensions)
{
	float3 VoxelPos = ScreenToVoxel(float3(0, 0, ViewDepth), VoxelDimensions);
	float NormalizedDepth = VoxelPos.z / VoxelDimensions.z;
	return NormalizedDepth;
}

float InversePerspectiveRemap(float NormalizedDepth, int3 VoxelDimensions)
{
	float3 VoxelPos = ScreenToVoxel(float3(0, 0, NormalizedDepth), VoxelDimensions);
	float ViewDepth = VoxelPos.z;
	return ViewDepth;
}

float3 ViewToVoxel(float3 ViewPos, int3 VoxelDimensions, float NearPlaneDepth, float FarPlaneDepth, float TanHalfFOV)
{
	//float4 ClipPos = mul(float4(ViewPos, 1), ViewToClip);
	//float3 ScreenPos = ClipPos.xyz / ClipPos.w;
	float2 AspectRatio = VoxelDimensions.xy / float(VoxelDimensions.x);
	float2 ScreenPos = ViewPos.xy / (TanHalfFOV * AspectRatio * ViewPos.z);

#if 0
	float NormalizedDepth = SquareDistanceRemap(ViewPos.z, NearPlaneDepth, FarPlaneDepth);
#elif 0
	float NormalizedDepth = ExponentialRemap(ViewPos.z, NearPlaneDepth, FarPlaneDepth);
#elif 1
	float NormalizedDepth = LinearRemap(ViewPos.z, NearPlaneDepth, FarPlaneDepth);
#else
	float NormalizedDepth = PerspectiveRemap(ViewPos.z, VoxelDimensions);
#endif

	float3 NDC = float3(ScreenPos.xy * 0.5 + 0.5, NormalizedDepth);
	float3 VoxelPos = NDC * VoxelDimensions;
	return VoxelPos;
}

float3 VoxelToView(float3 VoxelPos, int3 VoxelDimensions, float NearPlaneDepth, float FarPlaneDepth, float TanHalfFOV)
{
	float3 NDC = VoxelPos / VoxelDimensions;
	float2 ScreenPos = NDC.xy * 2.0 - 1.0;
#if 0
	float ViewDepth = InverseSquareDistanceRemap(NDC.z, NearPlaneDepth, FarPlaneDepth);
#elif 0
	float ViewDepth = InverseExponentialRemap(NDC.z, NearPlaneDepth, FarPlaneDepth);
#elif 1
	float ViewDepth = InverseLinearRemap(NDC.z, NearPlaneDepth, FarPlaneDepth);
#else
	float ViewDepth = InversePerspectiveRemap(NDC.z, VoxelDimensions);
#endif

	// TODO: Guarantee no view-space translation exists..
	float2 AspectRatio = VoxelDimensions.xy / float(VoxelDimensions.x);
	float3 ViewPos = float3(ScreenPos * TanHalfFOV * AspectRatio * ViewDepth, ViewDepth);
	return ViewPos;
}

float2 ClipRay(float3 Origin, float3 Direction, float TMin, float TMax, float4 Plane)
{
	float3 Normal = Plane.xyz;
	float d = Plane.w;
	float HitT = -(d + dot(Normal, Origin)) * rcp(dot(Normal, Direction));

	float2 RayT = float2(TMin, TMax);
	if ((HitT > RayT.x) && (HitT < RayT.y))
	{
		float SignedDistance = dot(Origin, Normal) + d;
		RayT = SignedDistance < 0 ? float2(HitT, RayT.y) : float2(RayT.x, HitT);
	}

	return RayT;
}