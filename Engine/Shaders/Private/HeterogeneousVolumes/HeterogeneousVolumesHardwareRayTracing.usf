// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesSparseVoxelUtils.ush"
#include "../Common.ush"
#include "../RayTracing/RayTracingCommon.ush"

#define SUPPORT_CONTACT_SHADOWS 0

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#if VIRTUAL_SHADOW_MAP
#include "../VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#include "HeterogeneousVolumesSparseVoxelUniformBufferUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesLightingUtils.ush"

#ifndef DIM_APPLY_SHADOW_TRANSMITTANCE
#define DIM_APPLY_SHADOW_TRANSMITTANCE 0
#endif // DIM_APPLY_SHADOW_TRANSMITTANCE

#define VOXEL_COUNT_MAX 6

#define SHADOW_BIT 0x80000000u

struct FSparseVoxelPayload
{
	float HitT[2];
	uint VoxelID;
	uint Data;
	float3 Transmittance;

	void SetMiss() { HitT[0] = -1; }
	bool IsMiss() { return HitT[0] < 0; }
	bool IsHit() { return !IsMiss(); }

	void SetShadowRay()
	{
		Data = (Data & ~SHADOW_BIT) | SHADOW_BIT;
	}
	bool IsShadowRay()
	{
		return (Data & SHADOW_BIT) != 0;
	}

	void SetTransmittance(float3 Transm)
	{
		Transmittance = Transm;
	}

	float3 GetTransmittance()
	{
		return Transmittance;
	}
};
CHECK_RT_PAYLOAD_SIZE(FSparseVoxelPayload)

FSparseVoxelPayload CreateSparseVoxelPayload()
{
	FSparseVoxelPayload Payload = (FSparseVoxelPayload)0;
	Payload.SetMiss();
	Payload.SetTransmittance(1.0);

	return Payload;
}

uint3 GetVolumeResolutionAtMip(uint MipLevel)
{
	uint3 VolumeResolution = GetVolumeResolution();
	return uint3(
		VolumeResolution.x >> MipLevel,
		VolumeResolution.y >> MipLevel,
		VolumeResolution.z >> MipLevel
	);
}

FVoxelData GetVoxelData(uint PrimitiveIndex, uint InstanceID)
{
	FVoxelDataPacked VoxelDataPacked = GetVoxelDataPacked(PrimitiveIndex);
	return DecodeVoxelData(VoxelDataPacked, GetVolumeResolution());
}

float3 GetObjectBoundsMin()
{
	float3 ObjectBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	return ObjectBoundsMin;
}

float3 GetObjectBoundsMax()
{
	float3 ObjectBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	return ObjectBoundsMax;
}

struct FSparseVoxelIntersectionAttributes
{
	float2 HitT;
};

FSparseVoxelIntersectionAttributes CreateVolumeIntersectionAttributes(float2 HitT)
{
	FSparseVoxelIntersectionAttributes Attr = (FSparseVoxelIntersectionAttributes)0;
	Attr.HitT = HitT;
	return Attr;
}

RAY_TRACING_ENTRY_CLOSEST_HIT(SparseVoxelsClosestHitShader,
	FSparseVoxelPayload, Payload,
	FSparseVoxelIntersectionAttributes, Attributes)
{
	FVoxelData VoxelData = GetVoxelData(PrimitiveIndex(), InstanceID());

	Payload.HitT[0] = Attributes.HitT.x;
	Payload.HitT[1] = Attributes.HitT.y;
	Payload.VoxelID = PrimitiveIndex();
}

RAY_TRACING_ENTRY_ANY_HIT(SparseVoxelsAnyHitShader,
	FSparseVoxelPayload, Payload,
	FSparseVoxelIntersectionAttributes, Attributes)
{
	FVoxelData VoxelData = GetVoxelData(PrimitiveIndex(), InstanceID());

	// Compute transmittance instead..
	Payload.HitT[0] = Attributes.HitT.x;
	Payload.HitT[1] = Attributes.HitT.y;
	Payload.VoxelID = PrimitiveIndex();

	if (Payload.IsShadowRay())
	{
		float3 Transmittance = Payload.GetTransmittance();

		float StepSize = GetShadowStepSize();
		float HitSpan = Payload.HitT[1] - Payload.HitT[0];
		int StepCount = max(HitSpan / StepSize, 1);
		StepSize = HitSpan / StepCount;
		float MipLevel = 0.0;

		for (int StepIndex = 0; StepIndex < StepCount; ++StepIndex)
		{
			float MarchT = StepIndex * StepSize;
			float3 ObjectPosition = ObjectRayOrigin() + ObjectRayDirection() * (Payload.HitT[0] + MarchT);
#if 1
			float3 UVW = (ObjectPosition - GetObjectBoundsMin()) / (GetObjectBoundsMax() - GetObjectBoundsMin());
			float3 Extinction = Texture3DSampleLevel(SparseVoxelUniformBuffer.ExtinctionTexture, SparseVoxelUniformBuffer.TextureSampler, UVW, MipLevel).rgb;
#else
			float3 WorldPosition = mul(float4(ObjectPosition, 1.0), GetLocalToWorld()).xyz;

			FVolumeSampleContext SampleContext = CreateVolumeSampleContext(ObjectPosition, WorldPosition, MipLevel);
			float3 Extinction = SampleExtinction(SampleContext);
#endif
			Transmittance *= exp(-Extinction * StepSize);
		}

		Payload.SetTransmittance(Transmittance);

		// Early ray termination
		float Epsilon = 1.0e-3;
		if (all(Transmittance < Epsilon))
		{
			AcceptHitAndEndSearch();
		}
		else
		{
			IgnoreHit();
		}
	}
}

RAY_TRACING_ENTRY_INTERSECTION(SparseVoxelsIntersectionShader)
{
	FVoxelData VoxelData = GetVoxelData(PrimitiveIndex(), InstanceID());
	FBox Box = GetVoxelBoundingBox(VoxelData, GetVolumeResolution(), GetObjectBoundsMin(), GetObjectBoundsMax());

	// Intersect ray against box
	float2 HitT = IntersectAABB(ObjectRayOrigin(), ObjectRayDirection(), RayTMin(), RayTCurrent(), Box.Min, Box.Max);
	if (HitT.y > HitT.x)
	{
		uint HitKind = 0;
		FSparseVoxelIntersectionAttributes IntersectionAttributes = CreateVolumeIntersectionAttributes(HitT);
		ReportHit(HitT.x, HitKind, IntersectionAttributes);
	}
}

RAY_TRACING_ENTRY_MISS(SparseVoxelsMissShader,
	FSparseVoxelPayload, Payload)
{
	Payload.SetMiss();
}

// These variables are only used by the ray gen shaders below
// Investigate why MipLevel isn't dead stripped when building the shaders above (see UE-218402).
#if RAYGENSHADER

// Scene
RaytracingAccelerationStructure TLAS;

// Ray data
int MaxStepCount;
int bJitter;

// Volume sampling data
int MipLevel;

// Output
RWTexture2D<float4> RWLightingTexture;

FRayDesc CreateVolumeRay(
	float3 Origin,
	float3 Direction,
	float TMin,
	float TMax
)
{
	FRayDesc Ray;
	Ray.Origin = Origin;
	Ray.Direction = Direction;
	Ray.TMin = TMin;
	Ray.TMax = TMax;

	return Ray;
}

void TraceVolumeRay(
	FRayDesc Ray,
	inout FSparseVoxelPayload Payload
)
{
	uint RayFlags = 0;
	uint InstanceInclusionMask = 0xFF;
	uint RayContributionToHitGroupIndex = 0;
	uint MultiplierForGeometryContributionToHitGroupIndex = 0;
	uint MissShaderIndex = 0;

	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToHitGroupIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		Payload
	);
}

void TraceVolumeShadowRay(
	FRayDesc Ray,
	inout FSparseVoxelPayload Payload
)
{
	Payload.SetShadowRay();
	TraceVolumeRay(Ray, Payload);
}

// h = [0, 1], s = [0 ,1], v = [0, 1]
// h: hue, 
// s: saturate
// v: value
float3 hsv2rgb(float h, float s, float v)
{
	h = 360.0 * h;

	float C = v * s;
	float X = C * (1 - abs((h / 60) % 2 - 1));
	float m = v - C;

	float3 RGB = 0;
	if (h >= 0 && h < 60)
	{
		RGB.r = C;
		RGB.g = X;
	}
	else if (h >= 60 && h < 120)
	{
		RGB.r = X;
		RGB.g = C;
	}
	else if (h >= 120 && h < 180)
	{
		RGB.g = C;
		RGB.b = X;
	}
	else if (h >= 180 && h < 240)
	{
		RGB.g = X;
		RGB.b = C;
	}
	else if (h >= 240 && h < 300)
	{
		RGB.r = X;
		RGB.b = C;
	}
	else if (h >= 300 && h < 360)
	{
		RGB.r = C;
		RGB.b = X;
	}

	RGB += m;
	return RGB;
}

float3 GetVoxelDebugColor(uint VoxelID)
{
	float Hue = (VoxelID % GetVolumeResolutionAtMip(MipLevel).x) / float(GetVolumeResolutionAtMip(MipLevel).x);
	return hsv2rgb(Hue, 1.0, 1.0);
}

float3 GetVoxelDebugColor(uint3 VoxelIndex)
{
	//float Hue = RandFast(VoxelIndex.yz);
	float Hue = float(Rand3DPCG32(VoxelIndex).x) / 0xFFFFFFFF;
	return hsv2rgb(Hue, 1.0, 1.0);
}

RAY_TRACING_ENTRY_RAYGEN(SparseVoxelsRayGenerationShader1)
{
	uint2 PixelCoord = DispatchRaysIndex().xy;
	if (any(PixelCoord.xy >= View.ViewSizeAndInvSize.xy))
	{
		return;
	}
	PixelCoord += View.ViewRectMin.xy;

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), GetMaxTraceDistance());
	DeviceZ = ConvertToDeviceZ(SceneDepth);

	float Jitter = bJitter ? InterleavedGradientNoise(PixelCoord, View.StateFrameIndexMod8) : 0.0;

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(PixelCoord + 0.5), PrimaryView.PreViewTranslation));
	float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(PixelCoord + 0.5, DeviceZ, 1)));
	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection); // SceneDepth

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float2 HitT = LineBoxIntersect(LocalRayOrigin, LocalRayEnd, LocalBoundsMin, LocalBoundsMax);

	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		float3 WorldRayStart = WorldRayOrigin + WorldRayDirection * HitT.x;
		float3 LocalRayStart = LocalRayOrigin + LocalRayDirection * HitT.x;

		// Convert normalized t-value intersection parameters to world-space units
		WorldRayDirection *= rcp(WorldRayLength);
		LocalRayDirection *= rcp(LocalRayLength);
		LocalRayLength *= HitSpan;

		// Jitter starting position to mitigate moire
		LocalRayStart += LocalRayDirection * Jitter;

		// Intersect against TLAS - HitT.x is real close to first intersection shader return value...
		FRayDesc Ray = CreateVolumeRay(WorldRayOrigin + WorldRayDirection * Jitter, WorldRayDirection, WorldRayLength * HitT.x, WorldRayLength * HitT.y);
		FSparseVoxelPayload Payload = CreateSparseVoxelPayload();
		TraceVolumeShadowRay(Ray, Payload);

		// Output..
		float3 Transmittance = Payload.GetTransmittance();
		float3 Opacity = 1.0 - Transmittance;
		RWLightingTexture[PixelCoord] = float4(Transmittance, Luminance(Opacity));
	}
}

// Lighting
int bApplyEmissionAndTransmittance;
int bApplyDirectLighting;
int bApplyShadowTransmittance;
int LightType;
float VolumetricScatteringIntensity;

// Shadowing
uint VirtualShadowMapId;
uint GetVirtualShadowMapId()
{
	return VirtualShadowMapId;
}

float3 ComputeTransmittanceHardwareRayTracing(
	float3 Origin,
	float3 Direction,
	float TMin,
	float TMax
)
{
	// TODO: Apply jitter?
	FRayDesc ShadowRay = CreateVolumeRay(Origin, Direction, TMin, TMax);
	FSparseVoxelPayload Payload = CreateSparseVoxelPayload();
	TraceVolumeShadowRay(ShadowRay, Payload);
	return Payload.GetTransmittance();
}

float EvalAmbientOcclusion(float3 WorldPosition)
{
	return 1.0;
}

// Cinematic feature options
bool UseInscatteringVolume()
{
	return true;
}

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
	return INDIRECT_LIGHTING_MODE_OFF;
}

bool IsOfflineRender()
{
	return false;
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
	return FOG_INSCATTERING_MODE_LINEAR_APPROX;
}

#define HARDWARE_RAY_TRACING 1
#include "HeterogeneousVolumesRayMarchingUtils.ush"

uint3 DispatchRaysToVoxelIndex(uint2 DispatchRaysIndex)
{
	uint3 VolumeResolution = GetLightingCacheResolution();
	uint3 VoxelIndex = uint3(DispatchRaysIndex.x,
		DispatchRaysIndex.y % LightingCacheResolution.y,
		DispatchRaysIndex.y / LightingCacheResolution.y);
	return VoxelIndex;
}

RWTexture3D<float3> RWLightingCacheTexture;

RAY_TRACING_ENTRY_RAYGEN(RenderLightingCacheWithPreshadingRGS)
{
	uint3 VoxelIndex = DispatchRaysToVoxelIndex(DispatchRaysIndex().xy);
	if (any(VoxelIndex >= LightingCacheResolution))
	{
		return;
	}

	float3 UVW = (VoxelIndex + 0.5) / float3(LightingCacheResolution);

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float3 LocalRayOrigin = UVW * GetLocalBoundsExtent() * 2.0 + LocalBoundsMin;
	float3 WorldRayOrigin = mul(float4(LocalRayOrigin, 1.0), GetLocalToWorld()).xyz;

	// Existence culling
	float MipLevel = 0.0f;
	FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalRayOrigin, WorldRayOrigin, MipLevel);
	float3 Extinction = SampleExtinction(SampleContext);
	float Epsilon = 1.0e-7;
	if (all(Extinction < Epsilon))
	{
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
		RWLightingCacheTexture[VoxelIndex] = 1.0f;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
		RWLightingCacheTexture[VoxelIndex] = 0.0f;
#endif // DIM_LIGHTING_CACHE_MODE
		return;
	}

	const FDeferredLightData LightData = InitDeferredLightFromUniforms(LightType, VolumetricScatteringIntensity);
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
	float3 L = LightData.Direction;
	float3 ToLight = L * GetMaxShadowTraceDistance();
	if (LightType != LIGHT_TYPE_DIRECTIONAL)
	{
		// Note: this function also permutes ToLight and L
		float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldRayOrigin, PrimaryView.PreViewTranslation);
		GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

		if (LightData.bRectLight)
		{
			FRect Rect = GetRect(ToLight, LightData);
		}
		else
		{
			FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
		}
	}

	float3 Transmittance = ComputeTransmittance(WorldRayOrigin, ToLight, MaxStepCount);
	RWLightingCacheTexture[VoxelIndex] = Transmittance;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
	float3 Inscattering = ComputeInscattering(WorldRayOrigin, LightData, LightType, MaxStepCount, CalcShadowBias(), bApplyShadowTransmittance);
	RWLightingCacheTexture[VoxelIndex] += Inscattering * View.PreExposure;
#endif // DIM_LIGHTING_CACHE_MODE
}

RAY_TRACING_ENTRY_RAYGEN(RenderSingleScatteringWithPreshadingRGS)
{
	float3 Radiance = 0.0;
	float3 Transmittance = 1.0;

	// TODO: Render tiles?

	uint2 PixelCoord = DispatchRaysIndex().xy;
	if (any(PixelCoord.xy >= View.ViewSizeAndInvSize.xy))
	{
		return;
	}
	PixelCoord += View.ViewRectMin.xy;

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), GetMaxTraceDistance());
	DeviceZ = ConvertToDeviceZ(SceneDepth);

	float Jitter = bJitter ? InterleavedGradientNoise(PixelCoord, View.StateFrameIndexMod8) : 0.0;

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(PixelCoord + 0.5), PrimaryView.PreViewTranslation));
	float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(PixelCoord + 0.5, DeviceZ, 1)));
	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection);
	WorldRayDirection /= WorldRayLength;

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();

	// Test bounding volume
	float2 BoundingHitT = IntersectAABB(LocalRayOrigin, LocalRayDirection, 0.0, LocalRayLength, LocalBoundsMin, LocalBoundsMax);
	bool bIntersectsVolume = (BoundingHitT.y - BoundingHitT.x > 0.0);
	if (!bIntersectsVolume)
	{
		return;
	}

	// Intersect against TLAS - HitT.x is real close to first intersection shader return value...
	FRayDesc Ray = CreateVolumeRay(WorldRayOrigin, WorldRayDirection, BoundingHitT.x, BoundingHitT.y);

	FSparseVoxelPayload Payload = CreateSparseVoxelPayload();
	TraceVolumeRay(Ray, Payload);

	float HitSpan = Payload.HitT[1] - Payload.HitT[0];
	while (Payload.IsHit() && Luminance(Transmittance) > 0.001)
	{
		FRayMarchingContext RayMarchingContext = CreateRayMarchingContext(
			// Local-space
			LocalRayOrigin,
			LocalRayDirection,
			Payload.HitT[0],
			Payload.HitT[1],
			// World-space
			WorldRayOrigin,
			WorldRayDirection,
			// Ray-step attributes
			Jitter,
			GetStepSize(),
			MaxStepCount,
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			DIM_APPLY_SHADOW_TRANSMITTANCE
		);

		const FDeferredLightData LightData = InitDeferredLightFromUniforms(LightType, VolumetricScatteringIntensity);
		uint StepCount = CalcStepCount(RayMarchingContext);

		float ScatterHitT = Payload.HitT[1];
		RayMarchSingleScattering(
			RayMarchingContext,
			LightData,
			LightType,
			StepCount,
			ScatterHitT,
			Radiance,
			Transmittance
		);

//#define DEBUG_VISUALIZATION 1
#if DEBUG_VISUALIZATION
		FVoxelData VoxelData = GetVoxelData(Payload.VoxelID, 0);
		if (View.GeneralPurposeTweak == 0)
		{
			// TODO: Remap Extinction to Transmittance
			float3 Extinction = 0.02 * (1.0 - GetVoxelDebugColor(VoxelData.VoxelIndex));
			Transmittance *= exp(-Extinction * HitSpan);
			if (!bApplyDirectLighting)
			{
				float3 Emission = 0.002 * GetVoxelDebugColor(VoxelData.VoxelIndex);
				Radiance += Emission * HitSpan * Transmittance;
			}
		}
#endif // DEBUG_VISUALIZATION

		//Ray = CreateVolumeRay(WorldRayOrigin, WorldRayDirection, Payload.HitT[1] + 0.001, WorldRayLength * HitT.y);
		Ray = CreateVolumeRay(WorldRayOrigin, WorldRayDirection, Payload.HitT[1] + 0.001, BoundingHitT.y);
		Payload = CreateSparseVoxelPayload();
		TraceVolumeRay(Ray, Payload);
		HitSpan = Payload.HitT[1] - Payload.HitT[0];
	}

	// Output..
	RWLightingTexture[PixelCoord] += float4(Radiance * View.PreExposure, Luminance(Transmittance));
}

#endif  // RAYGENSHADER
