// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../DeferredLightingCommon.ush"
#include "../LightGridCommon.ush"
#include "../LightDataUniforms.ush"

FDeferredLightData InitDeferredLightFromUniforms(uint LightType, float VolumetricScatteringIntensity)
{
    FDeferredLightData DeferredLightData = InitDeferredLightFromUniforms(LightType);
	DeferredLightData.Color *= VolumetricScatteringIntensity;

    return DeferredLightData;
}