// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "HeterogeneousVolumesTracingUtils.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

struct FScreenTile
{
	// TODO: This could be 16-bit..
	uint Id;
};

// Object data
float4x4 LocalToWorld;
float4x4 WorldToLocal;
float3 LocalBoundsOrigin;
float3 LocalBoundsExtent;

// Ray data
float MaxTraceDistance;

// Dispatch data
int3 GroupCount;
int DownsampleFactor;

// Output
RWBuffer<uint> RWNumScreenTilesBuffer;
RWStructuredBuffer<FScreenTile> RWScreenTileBuffer;

groupshared uint GSNumRayHits;

int2 GetDispatchSize()
{
	return ceil(View.ViewSizeAndInvSize.xy / float(DownsampleFactor));
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void ScreenTileClassificationCS(
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	if (all(GroupThreadId == 0))
	{
		GSNumRayHits = 0;
		RWNumScreenTilesBuffer[1] = 1;
		RWNumScreenTilesBuffer[2] = 1;
	}

	// Create screen ray
	if (any(DispatchThreadId.xy >= GetDispatchSize()))
	{
		return;
	}
	uint LinearIndex = DispatchThreadId.y * GroupCount.x * THREADGROUP_SIZE_2D + DispatchThreadId.x;
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy / DownsampleFactor;
	uint2 ViewPixelCoord = PixelCoord * DownsampleFactor + View.ViewRectMin.xy;
#if 0
	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), MaxTraceDistance);
#endif
	float SceneDepth = MaxTraceDistance;
	float DeviceZ = ConvertToDeviceZ(SceneDepth);

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(ViewPixelCoord + View.TemporalAAJitter.xy), PrimaryView.PreViewTranslation));
	float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord + View.TemporalAAJitter.xy, DeviceZ, 1)));
	//float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(ViewPixelCoord), PrimaryView.PreViewTranslation));
	//float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord, DeviceZ, 1)));
	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection);
	WorldRayDirection /= WorldRayLength;

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), WorldToLocal).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), WorldToLocal).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;

	float3 LocalBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
	float3 LocalBoundsMax = LocalBoundsOrigin + LocalBoundsExtent;

	// Intersect bounding volume
	float2 HitT = IntersectAABB(LocalRayOrigin, LocalRayDirection, 0.0, LocalRayLength, LocalBoundsMin, LocalBoundsMax);
	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		InterlockedAdd(GSNumRayHits, 1);
	}

	if (all(GroupThreadId == 0))
	{
		if (GSNumRayHits > 0)
		{
			uint BufferIndex = 0;
			InterlockedAdd(RWNumScreenTilesBuffer[0], 1, BufferIndex);

			FScreenTile ScreenTile;
			ScreenTile.Id = GroupId.y * GroupCount.x + GroupId.x;
			RWScreenTileBuffer[BufferIndex] = ScreenTile;
		}
	}
}

uint3 TextureResolution;
Texture3D<uint> UIntTexture3D;
RWTexture3D<float> RWFloatTexture3D;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void ConvertTexture3DFixedPointToFloatCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	float Value = float(UIntTexture3D[DispatchThreadId]);
	RWFloatTexture3D[DispatchThreadId] = Value / 65535.0;
}

Texture3D<float> ExistenceTexture;
SamplerState TextureSampler;
RWTexture3D<float> RWDilatedExistenceTexture;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void DilateExistenceMaskCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	if (any(DispatchThreadId > TextureResolution))
	{
		return;
	}

	float3 UVW = DispatchThreadId / (float3)TextureResolution;
	int MipLevel = 0;
	float Sample = ExistenceTexture.SampleLevel(
		TextureSampler,
		UVW,
		MipLevel).r;

	RWDilatedExistenceTexture[DispatchThreadId] = Sample > 0.0 ? 1.0 : 0.0;
}
