// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesSparseVoxelUtils.ush"
#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS 0
#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#ifndef SUPPORT_OVERLAPPING_VOLUMES
#define SUPPORT_OVERLAPPING_VOLUMES 0
#endif //

#ifndef ADAPTIVE_MARCH
#define ADAPTIVE_MARCH 0
#endif //

#ifndef REFERENCE_FAST_PATH
#define REFERENCE_FAST_PATH 0
#endif //

#ifndef DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP (AVSM_SAMPLE_MODE != 0)
#endif //

#ifndef USE_ANALYTIC_DERIVATIVES
#define USE_ANALYTIC_DERIVATIVES 0
#endif //

#ifndef DIM_WRITE_VELOCITY
#define DIM_WRITE_VELOCITY 0
#endif //DIM_WRITE_VELOCITY

#define USE_CAMERA_AVSM (DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP && (SUPPORT_OVERLAPPING_VOLUMES != 0))

#if SUPPORT_OVERLAPPING_VOLUMES
#define AVSM AVSMs.AVSM
#define CameraAVSM AVSMs.CameraAVSM
#endif // SUPPORT_OVERLAPPING_VOLUMES

#include "/Engine/Generated/Material.ush"
#include "HeterogeneousVolumesBeerShadowMapUtils.ush"
#include "HeterogeneousVolumesStochasticFiltering.ush"
#include "HeterogeneousVolumesLiveShadingUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesLightingUtils.ush"
#include "../BlueNoise.ush"

#if VIRTUAL_SHADOW_MAP
#include "../VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

// Object data
float4x4 LocalToWorld;
float4x4 WorldToLocal;
float3 LocalBoundsOrigin;
float3 LocalBoundsExtent;
int PrimitiveId;

// Volume data
int3 VoxelResolution;
int3 VoxelMin;
int3 VoxelMax;

// Lighting
int bHoldout;
int bApplyEmissionAndTransmittance;
int bApplyDirectLighting;
int bApplyShadowTransmittance;
int LightType;
float VolumetricScatteringIntensity;

// Shadowing
uint VirtualShadowMapId;
uint GetVirtualShadowMapId()
{
	return VirtualShadowMapId;
}

float ShadowStepSize;
float ShadowStepFactor;

// Atmosphere
int bApplyHeightFog;
int bApplyVolumetricFog;
int bCreateBeerShadowMap;

// Indirect
float IndirectInscatteringFactor;

// Ray data
float MaxTraceDistance;
float MaxShadowTraceDistance;
float StepSize;
float StepFactor;
int MaxStepCount;
int bJitter;
int StochasticFilteringMode;

// Dispatch data
int3 GroupCount;
int DownsampleFactor;

// Optional cinematic features
int bUseLightingCacheForInscattering;
int IndirectLightingMode;
int bWriteVelocity;
int AVSMSampleMode;
int bSupportsOverlappingVolumes;
int bIsOfflineRender;
int FogInscatteringMode;
int bUseAnalyticDerivatives;
int bUseReferenceFastPath;

// Output
RWTexture2D<float4> RWLightingTexture;
RWTexture2D<float4> RWVelocityTexture;
RWTexture2D<float> RWHoldoutTexture;
RWTexture2D<float4> RWBeerShadowMapTexture;

struct FDebugOutput
{
	int PrimitiveId;
	float3 LocalObjectBoundsMin;
	float3 LocalObjectBoundsMax;
	int Paddington;
	float4x4 LocalToWorld;
	float4x4 WorldToLocal;
};

RWStructuredBuffer<FDebugOutput> RWDebugOutputBuffer;

/*FDebugOutput CreateDebugOutput(
	inout FMaterialPixelParameters MaterialParameters,
	inout FPixelMaterialInputs PixelMaterialInputs)
{
	FDebugOutput DebugOutput = (FDebugOutput)0;

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(MaterialParameters);

	DebugOutput.PrimitiveId = MaterialParameters.PrimitiveId;
	DebugOutput.LocalObjectBoundsMin = PrimitiveData.LocalObjectBoundsMin.xyz;
	DebugOutput.LocalObjectBoundsMax = PrimitiveData.LocalObjectBoundsMax.xyz;
	DebugOutput.LocalToWorld = DFDemote(PrimitiveData.LocalToWorld);
	DebugOutput.WorldToLocal = DFDemote(PrimitiveData.WorldToLocal);

	return DebugOutput;
}*/

FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

float4x4 GetLocalToWorld()
{
	return LocalToWorld;
}

float4x4 GetWorldToLocal()
{
	return WorldToLocal;
}

float3 GetLocalBoundsOrigin()
{
	return LocalBoundsOrigin;
}

float3 GetLocalBoundsExtent()
{
	return LocalBoundsExtent;
}

int3 GetVolumeResolution()
{
	return VoxelResolution;
}

float GetStepSize()
{
	return StepSize;
}

float GetShadowStepSize()
{
	return ShadowStepSize;
}

float GetStepFactor()
{
	return StepFactor;
}

float GetShadowStepFactor()
{
	return ShadowStepFactor;
}

float GetMaxTraceDistance()
{
	return MaxTraceDistance;
}

float GetMaxShadowTraceDistance()
{
	return MaxShadowTraceDistance;
}

int GetStochasticFilteringMode()
{
	return StochasticFilteringMode;
}

int ShouldApplyHeightFog()
{
	return bApplyHeightFog;
}

int ShouldApplyVolumetricFog()
{
	return bApplyVolumetricFog;
}

float GetIndirectInscatteringFactor()
{
	return IndirectInscatteringFactor;
}

struct FVolumeSampleContext
{
	FMaterialPixelParameters MaterialParameters;
	FPixelMaterialInputs PixelMaterialInputs;
};

FVolumeSampleContext CreateVolumeSampleContext(
	float3 LocalPosition,
	float3 WorldPosition,
	float3 WorldPosition_DDX,
	float3 WorldPosition_DDY,
	float MipLevel
)
{
	ResolvedView = ResolveView();

	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.MaterialParameters = MakeInitializedMaterialPixelParameters();
	VolumeSampleContext.MaterialParameters.LWCData = MakeMaterialLWCData(VolumeSampleContext.MaterialParameters);
	//VolumeSampleContext.MaterialParameters.PrimitiveId = PrimitiveId;
	VolumeSampleContext.MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
	VolumeSampleContext.MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
	// TODO: Add object centroid to LWC.ObjectWorldPosition
	VolumeSampleContext.MaterialParameters.LWCData.LocalToWorld = WSPromote(GetLocalToWorld());
	VolumeSampleContext.MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(GetWorldToLocal());

#if USE_ANALYTIC_DERIVATIVES
	VolumeSampleContext.MaterialParameters.WorldPosition_DDX = WorldPosition_DDX;
	VolumeSampleContext.MaterialParameters.WorldPosition_DDY = WorldPosition_DDY;
	CalcPixelMaterialInputsAnalyticDerivatives(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#else
	CalcPixelMaterialInputs(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#endif

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0.0;
	float3 WorldPosition_DDY = 0.0;

	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int StochasticFilteringMode)
{
	// TODO: Use FilterWidth determine MipLevel
	//float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (StochasticFilteringMode == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;
	return CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	float3 Extinction = SampleExtinctionCoefficients(Context.PixelMaterialInputs);
	return Extinction;
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	float3 Emission = SampleEmissive(Context.PixelMaterialInputs);
	return Emission;
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = SampleAlbedo(Context.PixelMaterialInputs);
	return Albedo;
}

int3 AmbientOcclusionResolution;
Texture3D AmbientOcclusionTexture;

float EvalAmbientOcclusion(float3 WorldPosition)
{
	float3 LocalPosition = mul(float4(WorldPosition, 1.0), GetWorldToLocal()).xyz;
	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 UVW = saturate((LocalPosition - LocalBoundsMin) / (2.0 * GetLocalBoundsExtent()));
	float MipLevel = 0;

	return Texture3DSampleLevel(
		AmbientOcclusionTexture,
		View.SharedTrilinearClampedSampler,
		UVW,
		MipLevel).r;
}

#define HV_SCALABILITY_MODE_LOW 0
#define HV_SCALABILITY_MODE_HIGH 1
#define HV_SCALABILITY_MODE_EPIC 2
#define HV_SCALABILITY_MODE_CINEMATIC 3

#ifndef HV_SCALABILITY_MODE
#define HV_SCALABILITY_MODE HV_SCALABILITY_MODE_EPIC
#endif

// Cinematic feature options
bool UseInscatteringVolume()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return bUseLightingCacheForInscattering;
#else
	return true;
#endif
}

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return IndirectLightingMode;
#elif HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_EPIC
	return INDIRECT_LIGHTING_MODE_LIGHTING_CACHE;
#else
	return INDIRECT_LIGHTING_MODE_OFF;
#endif
}

bool ShouldWriteVelocity()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return DIM_WRITE_VELOCITY == 1;
#else
	return false;
#endif
}

bool IsOfflineRender()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return bIsOfflineRender;
#else
	return false;
#endif
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return FogInscatteringMode;
#else
	return FOG_INSCATTERING_MODE_LINEAR_APPROX;
#endif
}

#include "HeterogeneousVolumesRayMarchingUtils.ush"

uint bUseExistenceMask;
Texture3D ExistenceMaskTexture;
RWTexture3D<float3> RWLightingCacheTexture;

bool UseExistenceMask()
{
#if HV_SCALABILITY_MODE == HV_SCALABILITY_MODE_CINEMATIC
	return bUseExistenceMask;
#else
	return false;
#endif
}

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RenderLightingCacheWithLiveShadingCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	float3 VoxelIndex = DispatchThreadId + VoxelMin;
	if (any(DispatchThreadId > VoxelMax))
	{
		return;
	}

	float3 UVW = (VoxelIndex + 0.5) / float3(GetLightingCacheResolution());
	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float3 LocalRayOrigin = UVW * GetLocalBoundsExtent() * 2.0 + LocalBoundsMin;
	float3 WorldRayOrigin = mul(float4(LocalRayOrigin, 1.0), GetLocalToWorld()).xyz;

	// Existence culling
	float MipLevel = 0.0f;
	if (UseExistenceMask())
	{
		float Existence = ExistenceMaskTexture.SampleLevel(View.SharedBilinearClampedSampler, UVW, MipLevel).r;
		if (Existence <= 0.0)
		{
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
			RWLightingCacheTexture[VoxelIndex] = 1.0f;
#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
			RWLightingCacheTexture[VoxelIndex] = 0.0f;
#endif // DIM_LIGHTING_CACHE_MODE
			return;
		}
	}

	FVolumeSampleContext SampleContext;
	if (IsOfflineRender())
	{
		float3 FilterWidth = 1.0;
		int LinearIndex = VoxelIndex.x * (GetLightingCacheResolution().y * GetLightingCacheResolution().z) + VoxelIndex.y * GetLightingCacheResolution().z + VoxelIndex.z;
		float Rand = RandomSequenceCreate(LinearIndex, View.StateFrameIndex).Get1D();
		int StochasticFilteringMode = GetStochasticFilteringMode();
		SampleContext = CreateVolumeSampleContext(LocalRayOrigin, WorldRayOrigin, FilterWidth, MipLevel, Rand, StochasticFilteringMode);
	}
	else
	{
		SampleContext = CreateVolumeSampleContext(LocalRayOrigin, WorldRayOrigin, MipLevel);
	}

	float3 Extinction = SampleExtinction(SampleContext);
	float Epsilon = 1.0e-7;
	if (all(Extinction < Epsilon))
	{
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
		RWLightingCacheTexture[VoxelIndex] = 1.0f;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
		RWLightingCacheTexture[VoxelIndex] = 0.0f;
#endif // DIM_LIGHTING_CACHE_MODE
		return;
	}

	const FDeferredLightData LightData = InitDeferredLightFromUniforms(LightType, VolumetricScatteringIntensity);
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
	float3 L = LightData.Direction;
	float3 ToLight = L * GetMaxShadowTraceDistance();
	if (LightType != LIGHT_TYPE_DIRECTIONAL)
	{
		// Note: this function also permutes ToLight and L
		float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldRayOrigin, PrimaryView.PreViewTranslation);
		GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

		if (LightData.bRectLight)
		{
			FRect Rect = GetRect(ToLight, LightData);
		}
		else
		{
			FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
		}
	}

	float3 Transmittance = ComputeTransmittance(WorldRayOrigin, ToLight, MaxStepCount);
	RWLightingCacheTexture[VoxelIndex] = Transmittance;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
	float3 Inscattering = ComputeInscattering(WorldRayOrigin, LightData, LightType, MaxStepCount, CalcShadowBias(), bApplyShadowTransmittance);
	RWLightingCacheTexture[VoxelIndex] += Inscattering * View.PreExposure;
#endif // DIM_LIGHTING_CACHE_MODE
}

int2 GetDownsampledResolution(int2 Resolution, int Factor)
{
	return (Resolution + Factor - 1) / Factor;
}

int2 GetScaledViewRect()
{
	return GetDownsampledResolution(View.ViewSizeAndInvSize.xy, DownsampleFactor);
}

#if USE_CAMERA_AVSM

void RayMarchSingleScatteringAdaptive(
	inout FRayMarchingContext RayMarchingContext,
	inout FAVSM_Sampler4 TransmittanceSampler,
	uint StepCount,
	inout float3 Radiance,
	inout float3 Transmittance
)
{
	FAVSMSampleData SampleData = AVSM_Sampler4_GetSampleData0(TransmittanceSampler, 0);
	// Clip against first stored sample
	float WorldRayTMin = max(RayMarchingContext.LocalRayTMin * RayMarchingContext.LocalToWorldScale, SampleData.X);
	SampleData = AVSM_CreateSampleData(WorldRayTMin, AVSM_Sampler4_Eval(TransmittanceSampler, WorldRayTMin));

	float WorldShadowBias = CalcShadowBias();
	FAVSMSampleData PrevSampleData = AVSM_CreateSampleData(SampleData.X - WorldShadowBias, SampleData.Tau);

	FAVSMSampleData LastSampleData = AVSM_Sampler4_Last(TransmittanceSampler);
	// TODO: Sentinel values prevent efficient clipping against last stored sample
	//float WorldRayTMax = min(RayMarchingContext.LocalRayTMax * RayMarchingContext.LocalToWorldScale, LastSampleData.X);
	//LastSampleData = AVSM_CreateSampleData(WorldRayTMax, AVSM_Sampler4_Eval(TransmittanceSampler, WorldRayTMax));

	// Clip step count to stored transmittance range
	StepCount *= (SampleData.Tau - LastSampleData.Tau);
	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		float StepSize = (SampleData.X - PrevSampleData.X) / RayMarchingContext.LocalToWorldScale;
		float WorldHitT = SampleData.X;
		float LocalHitT = WorldHitT / RayMarchingContext.LocalToWorldScale;

		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * WorldHitT;
		FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, RayMarchingContext.MipLevel);

		// Evaluate transmittance
		Transmittance = PrevSampleData.Tau;

		// Radiance from emission
		if (RayMarchingContext.bApplyEmissionAndTransmittance)
		{
			float3 Emission = SampleEmission(SampleContext);
			Radiance += Emission * StepSize * Transmittance;
		}

		// Radiance from in-scattering
		float3 Extinction = SampleExtinction(SampleContext);
		if (RayMarchingContext.bApplyDirectLighting && (any(Extinction > 0)))
		{
			float3 RadianceSample = 0;

			// Evaluate in-scattering
			float3 Albedo = SampleAlbedo(SampleContext);
			if (any(Albedo > 0.0))
			{
				// Evaluate lighting cache
				float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
				float3 UVW = saturate((LocalPosition - LocalBoundsMin) / (2.0 * GetLocalBoundsExtent()));
				float3 Inscattering = SampleLightingCache(UVW, 0) * View.OneOverPreExposure;

				// Evaluate scattering
				float3 ScatteringCoefficient = Albedo * Extinction;
				float IsotropicPhase = 1.0 / (4.0 * PI);
				RadianceSample = Inscattering * ScatteringCoefficient * IsotropicPhase * Transmittance * StepSize;
			}

			Radiance += RadianceSample;
		}

		float Epsilon = 1.0e-7;
		if (all(Transmittance < Epsilon))
		{
			Transmittance = 0.0;
			break;
		}

		PrevSampleData = SampleData;
		float DeltaTau = rcp(StepCount);
		SampleData = AVSM_Sampler4_EvalInverse(TransmittanceSampler, SampleData.Tau - DeltaTau);

		// Guard against steps finer than shadowing bias
		if (SampleData.X - PrevSampleData.X < WorldShadowBias)
		{
			SampleData.X = PrevSampleData.X + WorldShadowBias;
			SampleData.Tau = AVSM_Sampler4_Eval(TransmittanceSampler, SampleData.X);
		}

		// Guard against adaptively stepping beyond t-max
		float WorldRayTMax = RayMarchingContext.LocalRayTMax * RayMarchingContext.LocalToWorldScale;
		if (SampleData.X > WorldRayTMax)
		{
			break;
		}
	}
}

#endif // USE_CAMERA_AVSM

void RenderSingleScatteringWithLiveShading(
	uint2 DispatchThreadId
)
{
	FDebugOutput DebugOutput = (FDebugOutput)0;

	// Create screen ray
	if (any(DispatchThreadId.xy >= GetScaledViewRect()))
	{
		return;
	}

	uint LinearIndex = DispatchThreadId.y * GroupCount.x * THREADGROUP_SIZE_2D + DispatchThreadId.x;
	uint2 ViewPixelCoord = DispatchThreadId.xy * DownsampleFactor + View.ViewRectMin.xy;
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy / DownsampleFactor;

	//float Epsilon = 1.0e-5;
	//if (RWLightingTexture[PixelCoord].a < Epsilon)
	//if (RWLightingTexture[PixelCoord].a <= 0.0)
	//{
	//	return;
	//}
	float3 Radiance = 0.0;
	float PrevTransmittance = RWLightingTexture[PixelCoord].a;
	float PrevOpacity = Luminance(1.0 - PrevTransmittance);

	FBeerShadowMapData BeerShadowMapData = InitializeBeerShadowMapData(PrevTransmittance, GetMaxTraceDistance());

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), GetMaxTraceDistance());
	DeviceZ = ConvertToDeviceZ(SceneDepth);

	float Jitter = bJitter ? BlueNoiseScalar(ViewPixelCoord, View.StateFrameIndexMod8) : 0.0;
	float2 ScreenJitter = IsOfflineRender() ? BlueNoiseVec2(ViewPixelCoord, View.StateFrameIndex) : View.TemporalAAJitter.xy;

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(ViewPixelCoord + ScreenJitter), PrimaryView.PreViewTranslation));
	float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord + ScreenJitter, DeviceZ, 1)));
	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection);
	WorldRayDirection /= WorldRayLength;

	// TODO: Analytical derivate evaluation is not currently supported under orthographic projection
	float3 WorldRayEnd_DDX = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord + ScreenJitter + float2(1.0, 0.0), DeviceZ, 1)));
	float3 WorldRayDirection_DDX = normalize(WorldRayEnd_DDX - WorldRayOrigin);
	float3 WorldRayEnd_DDY = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord + ScreenJitter + float2(0.0, 1.0), DeviceZ, 1)));
	float3 WorldRayDirection_DDY = normalize(WorldRayEnd_DDY - WorldRayOrigin);

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), WorldToLocal).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), WorldToLocal).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;

	float3 LocalRayEnd_DDX = mul(float4(WorldRayEnd_DDX, 1.0), WorldToLocal).xyz;
	float3 LocalRayDirection_DDX = normalize(LocalRayEnd_DDX - LocalRayOrigin);
	float3 LocalRayEnd_DDY = mul(float4(WorldRayEnd_DDY, 1.0), WorldToLocal).xyz;
	float3 LocalRayDirection_DDY = normalize(LocalRayEnd_DDY - LocalRayOrigin);

	float3 LocalBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
	float3 LocalBoundsMax = LocalBoundsOrigin + LocalBoundsExtent;

	// Intersect bounding volume
	float2 HitT = IntersectAABB(LocalRayOrigin, LocalRayDirection, 0.0, LocalRayLength, LocalBoundsMin, LocalBoundsMax);
	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		// March
		FRayMarchingContext RayMarchingContext = CreateRayMarchingContext(
			// Local-space
			LocalRayOrigin,
			LocalRayDirection,
			LocalRayDirection_DDX,
			LocalRayDirection_DDY,
			HitT.x,
			HitT.y,
			// World-space
			WorldRayOrigin,
			WorldRayDirection,
			WorldRayDirection_DDX,
			WorldRayDirection_DDY,
			// Ray-step attributes
			Jitter,
			CalcStepSize(LocalRayDirection),
			MaxStepCount,
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance
		);

		RayMarchingContext.RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex);

		const FDeferredLightData LightData = InitDeferredLightFromUniforms(LightType, VolumetricScatteringIntensity);
		uint StepCount = CalcStepCount(RayMarchingContext);

		float ScatterHitT = HitT.y;

		float3 Transmittance = float3(PrevTransmittance, PrevTransmittance, PrevTransmittance);

#if USE_CAMERA_AVSM
		FAdaptiveVolumetricShadowMap GlobalTransmittanceMap = GetAdaptiveVolumetricShadowMapForCamera();
		if (GlobalTransmittanceMap.bIsEmpty)
		{
			return;
		}

		// Define AVSM Sampler
		float MidpointT = (HitT.x + HitT.y) * 0.5;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * MidpointT;
		float3 TranslatedWorldPosition = DFHackToFloat(DFFastAdd(WorldPosition, PrimaryView.PreViewTranslation));

		int Face = 0;
		float2 Pixel = 0;
		float DepthOffset = 0;
		if (!AVSM_PixelAndDepth(GlobalTransmittanceMap, Face, TranslatedWorldPosition, Pixel, DepthOffset))
		{
			return;
		}

		FAVSM_Sampler4 TransmittanceSampler = AVSM_Sampler4_Create(GlobalTransmittanceMap, Face, Pixel, DepthOffset);

#if ADAPTIVE_MARCH
		StepCount = View.GeneralPurposeTweak2;
		RayMarchSingleScatteringAdaptive(
			RayMarchingContext,
			TransmittanceSampler,
			StepCount,
			Radiance,
			Transmittance
		);
#elif REFERENCE_FAST_PATH
		RayMarchSingleScatteringReferenceFastPath(
			RayMarchingContext,
			TransmittanceSampler,
			StepCount,
			Radiance,
			Transmittance
		);
#else
		RayMarchSingleScattering(
			RayMarchingContext,
			TransmittanceSampler,
			LightData,
			LightType,
			StepCount,
			ScatterHitT,
			Radiance,
			Transmittance
		);
#endif
		// Replace transmittance at WorldRayTMax with global transmittance at MaxTraceDistance
		//Transmittance = AVSM_Sampler4_Eval_NoInterpolation(TransmittanceSampler, WorldRayLength);
		Transmittance = AVSM_Sampler4_Eval(TransmittanceSampler, WorldRayLength);

#else
//	#if REFERENCE_FAST_PATH
//		RayMarchSingleScatteringReferenceFastPath(
//			RayMarchingContext,
//			StepCount,
//			Radiance,
//			Transmittance
//		);
//	#else
		RayMarchSingleScattering(
			RayMarchingContext,
			LightData,
			LightType,
			StepCount,
			ScatterHitT,
			Radiance,
			Transmittance
		);
//	#endif
#endif

		// Radiance
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
		if (bHoldout && View.bPrimitiveAlphaHoldoutEnabled)
		{
			Radiance *= 0;
		}
#endif
		RWLightingTexture[PixelCoord].rgb += Radiance * View.PreExposure;

		if (bApplyEmissionAndTransmittance)
		{
			float Opacity = Luminance(1.0 - Transmittance);
			RWLightingTexture[PixelCoord].a = Luminance(Transmittance);

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
			if (bHoldout && View.bPrimitiveAlphaHoldoutEnabled)
			{
				RWHoldoutTexture[PixelCoord] += Luminance(PrevTransmittance) - Luminance(Transmittance);
			}
#endif

			if (ShouldWriteVelocity())
			{
				bool bWriteVelocityLocal =
					(ScatterHitT < HitT.y) &&
					(PrevOpacity < 0.5) &&
					(Opacity > 0.5);

				if (bWriteVelocityLocal)
				{
					float WorldScatterHitT = ScatterHitT * RayMarchingContext.LocalToWorldScale;
					float3 Velocity = Calculate3DVelocityAtWorldPos(WorldRayOrigin, WorldRayDirection, WorldScatterHitT);
					RWVelocityTexture[PixelCoord] = EncodeVelocityToTexture(Velocity);
				}
			}
		}

		// TODO: Conditionalize around CameraAVSM
		if (bCreateBeerShadowMap)
		{
			BeerShadowMapData.T[0] = RayMarchingContext.LocalScatterT0 * RayMarchingContext.LocalToWorldScale;
			BeerShadowMapData.T[1] = RayMarchingContext.LocalScatterT1 * RayMarchingContext.LocalToWorldScale;
			BeerShadowMapData.Tau[0] = Luminance(RayMarchingContext.Transmittance0);
			BeerShadowMapData.Tau[1] = Luminance(Transmittance);
		}

		// Debug output
		//RWDebugOutputBuffer[LinearIndex] = DebugOutput;
	}

	if (bCreateBeerShadowMap)
	{
		RWBeerShadowMapTexture[PixelCoord] = PackBeerShadowMapData(
			CombineBeerShadowMapData(
				UnpackBeerShadowMapData(RWBeerShadowMapTexture[PixelCoord]),
				BeerShadowMapData,
				BEER_SHADOW_MAP_COMBINE_METHOD_EXPAND)
			// BEER_SHADOW_MAP_COMBINE_METHOD_FIRST)
		);
	}
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderSingleScatteringWithLiveShadingCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	ResolvedView = ResolveView();

	RenderSingleScatteringWithLiveShading(DispatchThreadId);
}

struct FScreenTile
{
	uint Id;
};

StructuredBuffer<FScreenTile> ScreenTileBuffer;

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderSingleScatteringWithLiveShadingIndirectCS(
	uint GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID
)
{
	ResolvedView = ResolveView();

	FScreenTile ScreenTile = ScreenTileBuffer[GroupId];
	uint2 GroupId_2D = uint2(ScreenTile.Id % GroupCount.x, ScreenTile.Id / GroupCount.x);
	uint2 DispatchThreadId = GroupId_2D * THREADGROUP_SIZE_2D + GroupThreadId;

	RenderSingleScatteringWithLiveShading(DispatchThreadId);
}

