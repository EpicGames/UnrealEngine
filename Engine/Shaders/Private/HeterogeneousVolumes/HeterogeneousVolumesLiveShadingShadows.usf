// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS 0

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#include "/Engine/Generated/Material.ush"
#include "HeterogeneousVolumesStochasticFiltering.ush"
#include "HeterogeneousVolumesLiveShadingUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapUtils.ush"
#include "HeterogeneousVolumesBeerShadowMapUtils.ush"
#include "../BlueNoise.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

#define SHADOW_MAP_TYPE_AVSM 0
#define SHADOW_MAP_TYPE_BEER 1

#ifndef SHADOW_MAP_TYPE
#define SHADOW_MAP_TYPE SHADOW_MAP_TYPE_AVSM
#endif // SHADOW_MAP_TYPE

// Object data
float4x4 LocalToWorld;
float4x4 WorldToLocal;
float3 LocalBoundsOrigin;
float3 LocalBoundsExtent;
int PrimitiveId;

// Volume data
int3 VoxelResolution;

// Ray data
float MaxTraceDistance;
float MaxShadowTraceDistance;
float StepSize;
float StepFactor;
float ShadowStepSize;
float ShadowStepFactor;
int MaxStepCount;
int bJitter;
int StochasticFilteringMode;

// Shadow data
int NumShadowMatrices;
float4x4 TranslatedWorldToShadow[6];
float4x4 ShadowToTranslatedWorld[6];
float4 TranslatedWorldOrigin[6];

int2 ShadowResolution;
int MaxSampleCount;
float AbsoluteErrorThreshold;
float RelativeErrorThreshold;

// Dispatch data
int3 GroupCount;

// Debug data
int ShadowDebugTweak;

// Output
RWBuffer<int> RWVolumetricShadowLinkedListAllocatorBuffer;
RWStructuredBuffer<uint2> RWVolumetricShadowLinkedListBuffer;
RWTexture2D<float4> RWBeerShadowMapTexture;

FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

float4x4 GetLocalToWorld()
{
	return LocalToWorld;
}

float4x4 GetWorldToLocal()
{
	return WorldToLocal;
}

float3 GetLocalBoundsOrigin()
{
	return LocalBoundsOrigin;
}

float3 GetLocalBoundsExtent()
{
	return LocalBoundsExtent;
}

int3 GetVolumeResolution()
{
	return VoxelResolution;
}

float GetShadowStepSize()
{
	return ShadowStepSize;
}

float GetShadowStepFactor()
{
	return ShadowStepFactor;
}

float GetStepSize()
{
	return GetShadowStepSize();
}

float GetStepFactor()
{
	return GetShadowStepFactor();
}

float GetMaxTraceDistance()
{
	return MaxTraceDistance;
}

float GetMaxShadowTraceDistance()
{
	return MaxShadowTraceDistance;
}

int GetStochasticFilteringMode()
{
	return StochasticFilteringMode;
}

float EvalAmbientOcclusion(float3 WorldPosition)
{
	return 0.0;
}

float GetIndirectInscatteringFactor()
{
	return 0.0;
}

struct FVolumeSampleContext
{
	FMaterialPixelParameters MaterialParameters;
	FPixelMaterialInputs PixelMaterialInputs;
};

FVolumeSampleContext CreateVolumeSampleContext(
	float3 LocalPosition,
	float3 WorldPosition,
	float3 WorldPosition_DDX,
	float3 WorldPosition_DDY,
	float MipLevel
)
{
	ResolvedView = ResolveView();

	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.MaterialParameters = MakeInitializedMaterialPixelParameters();
	VolumeSampleContext.MaterialParameters.LWCData = MakeMaterialLWCData(VolumeSampleContext.MaterialParameters);
	//VolumeSampleContext.MaterialParameters.PrimitiveId = PrimitiveId;
	VolumeSampleContext.MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
	VolumeSampleContext.MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
	// TODO: Add object centroid to LWC.ObjectWorldPosition
	VolumeSampleContext.MaterialParameters.LWCData.LocalToWorld = WSPromote(GetLocalToWorld());
	VolumeSampleContext.MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(GetWorldToLocal());

#if USE_ANALYTIC_DERIVATIVES
	VolumeSampleContext.MaterialParameters.WorldPosition_DDX = WorldPosition_DDX;
	VolumeSampleContext.MaterialParameters.WorldPosition_DDY = WorldPosition_DDY;
	CalcPixelMaterialInputsAnalyticDerivatives(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#else
	CalcPixelMaterialInputs(VolumeSampleContext.MaterialParameters, VolumeSampleContext.PixelMaterialInputs);
#endif

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0.0;
	float3 WorldPosition_DDY = 0.0;

	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int StochasticFilteringMode)
{
	// TODO: Use FilterWidth determine MipLevel
	//float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (StochasticFilteringMode == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;
	return CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	float3 Extinction = SampleExtinctionCoefficients(Context.PixelMaterialInputs);
	return Extinction;
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	float3 Emission = SampleEmissive(Context.PixelMaterialInputs);
	return Emission;
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = SampleAlbedo(Context.PixelMaterialInputs);
	return Albedo;
}

uint GetLinearIndex(uint Face, uint2 PixelCoord)
{
	uint LinearIndex = Face * ShadowResolution.x * ShadowResolution.y + PixelCoord.y * ShadowResolution.x + PixelCoord.x;
	return LinearIndex;
}

uint2 GetTexture2DIndex(uint Face, uint2 PixelCoord)
{
	int2 Texture2DIndex = uint2(Face * ShadowResolution.x + PixelCoord.x, PixelCoord.y);
	return Texture2DIndex;
}

// Cinematic feature options
bool UseInscatteringVolume()
{
	return true;
}

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
	return INDIRECT_LIGHTING_MODE_OFF;
}

bool IsOfflineRender()
{
	return false;
}

int ShouldApplyHeightFog()
{
	return false;
}

int ShouldApplyVolumetricFog()
{
	return false;
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
	return FOG_INSCATTERING_MODE_OFF;
}

#include "HeterogeneousVolumesRayMarchingUtils.ush"

int FindSampleIndexWithLowestError(
	int PixelIndex
)
{
	// Find sample, whose removal would mark the lowest overall geometric error
	int LowestErrorIndex = 0;
	float LowestError = 1.0e6;
	for (int Index = 1; Index < MaxSampleCount - 1; ++Index)
	{
		int PrevIndex = AVSM_GetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]);
		int NextIndex = AVSM_GetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]);

		FAVSMSampleData Data[3] = {
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevIndex]),
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]),
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextIndex])
		};

		float Error = abs(
			(Data[2].X - Data[0].X) * (Data[2].Tau - Data[1].Tau) -
			(Data[2].X - Data[1].X) * (Data[2].Tau - Data[0].Tau)
		);

		if (Error < LowestError)
		{
			LowestError = Error;
			LowestErrorIndex = Index;
		}
	}

	return LowestErrorIndex;
}

void AddSample(FAVSMSampleData SampleData, int PixelIndex, inout int PrevSampleIndex, inout int SampleIndex)
{
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], SampleData);
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], PrevSampleIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], SampleIndex + 1);

	PrevSampleIndex = SampleIndex;
	SampleIndex = SampleIndex + 1;
}

void ReplaceSample(
	int PixelIndex,
	int SampleIndex,
	FAVSMSampleData SampleData,
	int PrevPtrIndex,
	int NextPtrIndex
)
{
	// Remove 
	{
		int PrevIndex = AVSM_GetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex]);
		int NextIndex = AVSM_GetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex]);

		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevIndex], NextIndex);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextIndex], PrevIndex);
	}

	// Replace
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], SampleData);
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], PrevPtrIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], NextPtrIndex);

	// Update neighborhood
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextPtrIndex], SampleIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevPtrIndex], SampleIndex);
}

void AddOrReplaceSample(FAVSMSampleData Data, int PixelIndex, inout int PrevSampleIndex, inout int SampleIndex)
{
	// Add
	if (SampleIndex < MaxSampleCount - 1)
	{
		AddSample(Data, PixelIndex, PrevSampleIndex, SampleIndex);
	}
	// Or replace
	else
	{
#if USE_AVSM_COMPRESSION
		// Insert fictious sample at MaxSampleCount - 1 to compute error
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], Data);
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevSampleIndex], MaxSampleCount - 1);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], PrevSampleIndex);

		int LowestErrorIndex = FindSampleIndexWithLowestError(PixelIndex);
		if (LowestErrorIndex > 0)
		{
			if (LowestErrorIndex == PrevSampleIndex)
			{
				AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevSampleIndex], Data);
			}
			else
			{
				ReplaceSample(PixelIndex, LowestErrorIndex, Data, PrevSampleIndex, MaxSampleCount - 1);
			}

			PrevSampleIndex = LowestErrorIndex;
		}
#else
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], Data);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], PrevSampleIndex);

		PrevSampleIndex = MaxSampleCount - 1;
#endif
	}
}

int CameraDownsampleFactor;

float3 RayMarchTransmittance(
	inout FRayMarchingContext RayMarchingContext,
	int PixelIndex,
	inout FBeerShadowMapData BeerShadowMapData
)
{
	float3 LocalBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
	float3 LocalBoundsMax = LocalBoundsOrigin + LocalBoundsExtent;

	float3 Transmittance = 1;
	InitializeBeerShadowMapData(1.0, GetMaxTraceDistance());

	uint StepCount = CalcStepCount(RayMarchingContext);

	float3 ExpectedExtinction = 0.0;
	float3 ExpectedTransmittance = 1.0;
	int PrevElementIndex = AVSM_NULL_PTR;
	int ElementIndex = 0;

	float StepSize = RayMarchingContext.StepSize;
	float JitterSize = StepSize * RayMarchingContext.Jitter;

	float LocalHitT = RayMarchingContext.LocalRayTMin;
	float WorldHitT = LocalHitT * RayMarchingContext.LocalToWorldScale;
	float LastWorldHitT = WorldHitT;
	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		LocalHitT = min(RayMarchingContext.LocalRayTMin + StepSize * (RayMarchingContext.Jitter + StepIndex), RayMarchingContext.LocalRayTMax);
		if (StepIndex == StepCount - 1)
		{
			float UnjitteredLocalHitT = LocalHitT - JitterSize;
			StepSize = RayMarchingContext.LocalRayTMax - UnjitteredLocalHitT;
		}

		WorldHitT = LocalHitT * RayMarchingContext.LocalToWorldScale;
		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * WorldHitT;

		FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, RayMarchingContext.MipLevel);
		float3 Extinction = SampleExtinction(SampleContext);
		Transmittance *= exp(-Extinction * StepSize);

		if (any(Extinction > 0.0))
		{
			LastWorldHitT = WorldHitT;
			bool bFirstEntry = (ElementIndex == 0);

#if SHADOW_MAP_TYPE == SHADOW_MAP_TYPE_BEER
			if (bFirstEntry)
			{
				BeerShadowMapData.T[0] = WorldHitT;
				BeerShadowMapData.Tau[0] = Luminance(Transmittance);
				ElementIndex = 1;
			}
#else // SHADOW_MAP_TYPE == SHADOW_MAP_TYPE_AVSM
			if (bFirstEntry)
			{
				FAVSMSampleData SampleData = AVSM_CreateSampleData(WorldHitT, Transmittance);
				AddSample(SampleData, PixelIndex, PrevElementIndex, ElementIndex);

				// Set expected values
				ExpectedExtinction = Extinction;
				ExpectedTransmittance = Transmittance;
			}
			else
			{
				// If extrapolated transmittance is beyond relative error tolerance
				ExpectedTransmittance *= exp(-ExpectedExtinction * StepSize);
				float AbsoluteError = length(Transmittance - ExpectedTransmittance);
				float RelativeError = length((Transmittance - ExpectedTransmittance) / ExpectedTransmittance);
				if ((AbsoluteError > AbsoluteErrorThreshold) && (RelativeError > RelativeErrorThreshold))
				{
					FAVSMSampleData SampleData = AVSM_CreateSampleData(WorldHitT, Transmittance);
					AddOrReplaceSample(SampleData, PixelIndex, PrevElementIndex, ElementIndex);

					// Reset expected values
					ExpectedExtinction = Extinction;
					ExpectedTransmittance = Transmittance;
				}
			}
#endif
		}

		// Early termination
		float Epsilon = 1.0e-7;
		if (all(Transmittance < Epsilon))
		{
			Transmittance = 0.0;
			break;
		}
	}

#if SHADOW_TYPE == SHADOW_TYPE_AVSM
	// Stitch up the previous entry
	if (PrevElementIndex != AVSM_NULL_PTR)
	{
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevElementIndex], ElementIndex);

	}

	// Last Entry
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], AVSM_CreateSampleData(LastWorldHitT, Transmittance));
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], PrevElementIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], AVSM_NULL_PTR);
#endif

#if SHADOW_MAP_TYPE == SHADOW_MAP_TYPE_BEER
	BeerShadowMapData.T[1] = LastWorldHitT;
	BeerShadowMapData.Tau[1] = saturate(Luminance(Transmittance));
#endif

	return Transmittance;
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderVolumetricShadowMapForLightWithLiveShadingCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	// Create screen ray
	if (any(DispatchThreadId.xy >= ShadowResolution))
	{
		return;
	}

	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	uint LinearIndex = GetLinearIndex(Face, PixelCoord);
	int PixelIndex = LinearIndex * MaxSampleCount;

	// Generate a light ray
	float2 ScreenJitter = IsOfflineRender() ? BlueNoiseVec2(PixelCoord, View.StateFrameIndex) : View.TemporalAAJitter.xy;
	//float2 ShadowUV = float2(PixelCoord + 0.5f) / float2(ShadowResolution);
	float2 ShadowUV = float2(PixelCoord + ScreenJitter) / float2(ShadowResolution);

	float WorldRayLength = GetMaxTraceDistance();
#if USE_CAMERA_SCENE_DEPTH
	float2 ViewPixelCoord = (PixelCoord + View.TemporalAAJitter.xy) * CameraDownsampleFactor + View.ViewRectMin.xy / CameraDownsampleFactor;
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Project ray through view transform to trim ray-length by opaque depth
	FDFVector3 WorldViewRayOrigin = DFFastSubtract(GetTranslatedWorldCameraPosFromView(ViewPixelCoord), PrimaryView.PreViewTranslation);
	FDFVector3 WorldViewRayEnd = SvPositionToWorld(float4(ViewPixelCoord, DeviceZ, 1));
	float3 WorldViewRayDirection = DFFastSubtractDemote(WorldViewRayEnd, WorldViewRayOrigin);
	float WorldViewRayLength = length(WorldViewRayDirection);

	WorldRayLength = min(WorldRayLength, WorldViewRayLength);
#endif

	const float NearScreenZ = 1.0;
	const float FarScreenZ = 0.0;

	float4 LightRayStartAsFloat4 = mul(float4(ShadowUV, NearScreenZ, 1), ShadowToTranslatedWorld[Face]);
	float4 LightRayEndAsFloat4 = mul(float4(ShadowUV, FarScreenZ, 1), ShadowToTranslatedWorld[Face]);
	float3 LightRayStart = LightRayStartAsFloat4.xyz * rcp(LightRayStartAsFloat4.w);
	float3 LightRayEnd = LightRayEndAsFloat4.xyz * rcp(LightRayEndAsFloat4.w);

	float3 WorldRayOrigin = DFFastSubtractDemote(LightRayStart, PrimaryView.PreViewTranslation);
	float3 WorldRayEnd = DFFastSubtractDemote(LightRayEnd, PrimaryView.PreViewTranslation);
	float3 WorldRayDirection = normalize(WorldRayEnd - WorldRayOrigin);
	WorldRayEnd = WorldRayOrigin + WorldRayDirection * WorldRayLength;

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), WorldToLocal).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), WorldToLocal).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;

	float3 LocalBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
	float3 LocalBoundsMax = LocalBoundsOrigin + LocalBoundsExtent;

	float3 Transmittance = 1;
	FBeerShadowMapData BeerShadowMapData = InitializeBeerShadowMapData(1.0, GetMaxTraceDistance());

	// Intersect bounding volume
	float2 HitT = IntersectAABB(LocalRayOrigin, LocalRayDirection, 0.0, LocalRayLength, LocalBoundsMin, LocalBoundsMax);
	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		float Jitter = bJitter ? BlueNoiseScalar(PixelCoord, View.StateFrameIndexMod8) : 0.0;
		int bApplyEmissionAndTransmittance = 0;
		int bApplyDirectLighting = 0;
		int bApplyShadowTransmittance = 0;

		FRayMarchingContext RayMarchingContext = CreateRayMarchingContext(
			// Local-space
			LocalRayOrigin,
			LocalRayDirection,
			HitT.x,
			HitT.y,
			// World-space
			WorldRayOrigin,
			WorldRayDirection,
			// Ray-step attributes
			Jitter,
			CalcShadowStepSize(WorldRayDirection),
			MaxStepCount,
			//MaxSampleCount, // Replace max-steps with max-samples..
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance
		);
		RayMarchingContext.RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex);

		Transmittance = RayMarchTransmittance(RayMarchingContext, PixelIndex, BeerShadowMapData);
	}
#if SHADOW_MAP_TYPE == SHADOW_MAP_TYPE_AVSM
	else
#endif
	{
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_CreateSampleData(0.0, Transmittance));
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
	}

#if SHADOW_MAP_TYPE == SHADOW_MAP_TYPE_BEER
	uint2 BeerShadowMapCoord = GetTexture2DIndex(Face, PixelCoord);
	RWBeerShadowMapTexture[BeerShadowMapCoord] = PackBeerShadowMapData(
		CombineBeerShadowMapData(
			UnpackBeerShadowMapData(RWBeerShadowMapTexture[BeerShadowMapCoord]),
			BeerShadowMapData,
			BEER_SHADOW_MAP_COMBINE_METHOD_EXPAND)
		// BEER_SHADOW_MAP_COMBINE_METHOD_FIRST)
	);
#endif
}
