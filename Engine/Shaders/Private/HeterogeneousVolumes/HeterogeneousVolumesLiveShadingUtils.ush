// Copyright Epic Games, Inc. All Rights Reserved.

// Change this to force recompilation of all shaders referencing this file.
#pragma message("UESHADERMETADATA_VERSION 80584AA0-BFD9-4021-BE67-273077B48601")

FPrimitiveSceneData GetPrimitiveData(
	uint PrimitiveId,
	float4x4 LocalToWorld,
	float4x4 WorldToLocal,
	float3 LocalBoundsOrigin,
	float3 LocalBoundsExtent
)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);

	// Replace local-space transforms..
	PrimitiveData.LocalToWorld = DFPromote(LocalToWorld);
	PrimitiveData.WorldToLocal = DFPromoteInverse(WorldToLocal);
	PrimitiveData.LocalObjectBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
	PrimitiveData.LocalObjectBoundsMax = LocalBoundsOrigin + LocalBoundsExtent;

	return PrimitiveData;
}

FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters, float4x4 LocalToWorld, float4x4 WorldToLocal, float3 LocalBoundsOrigin, float3 LocalBoundsExtent)
{
	return GetPrimitiveData(Parameters.PrimitiveId, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters, float4x4 LocalToWorld, float4x4 WorldToLocal, float3 LocalBoundsOrigin, float3 LocalBoundsExtent)
{
	return GetPrimitiveData(Parameters.PrimitiveId, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

// Stolen from VolumetricFog
float3 SampleExtinctionCoefficients(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 Extinction = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Extinction = VOLUMETRICFOGCLOUD_EXTINCTION(BSDF).rgb;
#else
#if !MATERIAL_SHADINGMODEL_UNLIT
	Extinction = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs).rgb;
#endif
#endif
	return clamp(Extinction, 0.0f, 65000.0f);
}

float3 SampleEmissive(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 EmissiveColor = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	EmissiveColor = BSDF_GETEMISSIVE(BSDF);
#else
	EmissiveColor = GetMaterialEmissiveRaw(PixelMaterialInputs);
#endif
	return clamp(EmissiveColor, 0.0f, 65000.0f);
}

float3 SampleAlbedo(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 Albedo = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Albedo = VOLUMETRICFOGCLOUD_ALBEDO(BSDF);
#else
#if !MATERIAL_SHADINGMODEL_UNLIT
	Albedo = GetMaterialBaseColor(PixelMaterialInputs);
#endif
#endif
	return saturate(Albedo) * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
}
