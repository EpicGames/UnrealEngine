// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS 0
#define EYE_ADAPTATION_LOOSE_PARAMETERS 1
#define SQRT_THREE 1.7320508075688772

#include "/Engine/Generated/Material.ush"
#include "../BlueNoise.ush"
#include "../PositionReconstructionCommon.ush"
#include "../MortonCode.ush"
#include "../ComputeShaderUtils.ush"
#include "../Hash.ush"
#include "../HashTable.ush"
#include "HeterogeneousVolumesLiveShadingUtils.ush"


#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

// Object data
float4x4 LocalToWorld;
float4x4 WorldToLocal;
float3 LocalBoundsOrigin;
float3 LocalBoundsExtent;
int PrimitiveId;
int bJitter;
int bEvaluateEmission;
int bEvaluateAlbedo;

// Volume data
int3 VolumeResolution;

// Output
RWTexture3D<float3> RWExtinctionTexture;
RWTexture3D<float3> RWEmissionTexture;
RWTexture3D<float3> RWAlbedoTexture;

FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters, LocalToWorld, WorldToLocal, LocalBoundsOrigin, LocalBoundsExtent);
}

uint2 VoxelIndex2D(uint3 VoxelIndex)
{
	return uint2(VoxelIndex.y * VolumeResolution.x + VoxelIndex.x, VoxelIndex.z);
}

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void HeterogeneousVolumesBakeMaterialCS(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	ResolvedView = ResolveView();

	uint3 VoxelIndex = DispatchThreadId;
	if (all(VoxelIndex < VolumeResolution))
	{
		float3 WorldOrigin = mul(float4(0.0, 0.0, 0.0, 1.0), LocalToWorld).xyz;
		float3 WorldVoxelPos = mul(float4(1.0, 1.0, 1.0, 1.0), LocalToWorld).xyz;
		float WorldVoxelVolume = length(WorldVoxelPos - WorldOrigin);

		float3 Jitter = 0.5;
		if (bJitter)
		{
			Jitter.xy = BlueNoiseVec2(VoxelIndex2D(VoxelIndex), View.StateFrameIndex);
			Jitter.z = BlueNoiseScalar(VoxelIndex2D(VoxelIndex), View.StateFrameIndex);
		}

		float3 UVW = clamp((VoxelIndex + Jitter) / float3(VolumeResolution), 0, VolumeResolution);
		float3 LocalBoundsMin = LocalBoundsOrigin - LocalBoundsExtent;
		float3 LocalPosition = UVW * 2.0 * LocalBoundsExtent + LocalBoundsMin;
		float3 WorldPosition = mul(float4(LocalPosition, 1.0), LocalToWorld).xyz;

		// Setup evaluation context
		FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
		MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);
		MaterialParameters.PrimitiveId = PrimitiveId;
		MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
		MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
		// TODO: Add object centroid to LWC.ObjectWorldPosition
		MaterialParameters.LWCData.LocalToWorld = WSPromote(LocalToWorld);
		MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(WorldToLocal);

		// Evaluate material graph
		FPixelMaterialInputs PixelMaterialInputs;
		CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

		// Extract volume rendering coefficients
		if (bEvaluateAlbedo)
		{
			RWAlbedoTexture[VoxelIndex] = SampleAlbedo(PixelMaterialInputs);
		}

		if (bEvaluateEmission)
		{
			RWEmissionTexture[VoxelIndex] = SampleEmissive(PixelMaterialInputs);
		}

		RWExtinctionTexture[VoxelIndex] = SampleExtinctionCoefficients(PixelMaterialInputs);
	}
}

// Global transmittance volume

#include "HeterogeneousVolumesVoxelGridTypes.ush"
#include "HeterogeneousVolumesVoxelGridUtils.ush"

int3 TopLevelGridResolution;
float3 TopLevelGridWorldBoundsMin;
float3 TopLevelGridWorldBoundsMax;

float3 PrimitiveWorldBoundsMin;
float3 PrimitiveWorldBoundsMax;

int BottomLevelGridBufferSize;

void CalcVoxelBounds(float3 VoxelIndex, inout float3 VoxelBoundsMin, inout float3 VoxelBoundsMax)
{
	float3 TopLevelGridWorldBoundsExtent = TopLevelGridWorldBoundsMax - TopLevelGridWorldBoundsMin;
	VoxelBoundsMin = TopLevelGridWorldBoundsMin + TopLevelGridWorldBoundsExtent * (VoxelIndex / TopLevelGridResolution);
	VoxelBoundsMax = TopLevelGridWorldBoundsMin + TopLevelGridWorldBoundsExtent * ((VoxelIndex + 1) / TopLevelGridResolution);
}

float3 GetVoxelCenter(float3 VoxelIndex, inout float3 VoxelBoundsMin, inout float3 VoxelBoundsMax)
{
	float3 TopLevelGridWorldBoundsExtent = TopLevelGridWorldBoundsMax - TopLevelGridWorldBoundsMin;
	return TopLevelGridWorldBoundsMin + TopLevelGridWorldBoundsExtent * ((VoxelIndex + 0.5) / TopLevelGridResolution);
}

bool PrimitiveIntersectsVoxel(float3 VoxelBoundsMin, float3 VoxelBoundsMax)
{
	if (any(PrimitiveWorldBoundsMin > VoxelBoundsMax) || any(VoxelBoundsMin > PrimitiveWorldBoundsMax))
	{
		return false;
	}

	return true;
}

bool WorldPositionIntersectsPrimitive(float3 WorldPosition)
{
	return all(WorldPosition >= PrimitiveWorldBoundsMin) && all(WorldPosition <= PrimitiveWorldBoundsMax);
}

bool LocalPositionIntersectsPrimitive(float3 LocalPosition)
{
	float3 LocalMinPosition = LocalBoundsOrigin - LocalBoundsExtent;
	float3 LocalMaxPosition = LocalBoundsOrigin + LocalBoundsExtent;
	return all(LocalPosition >= LocalMinPosition) && all(LocalPosition <= LocalMaxPosition);
}

struct FRasterTileData
{
	uint TopLevelGridLinearIndex;
	uint BottomLevelGridLinearOffset;
};

Buffer<uint> RasterTileAllocatorBuffer;
StructuredBuffer<FRasterTileData> RasterTileBuffer;

RWStructuredBuffer<FTopLevelGridData> RWTopLevelGridBuffer;
RWStructuredBuffer<FTopLevelGridData> RWIndirectionGridBuffer;

RWBuffer<uint> RWBottomLevelGridAllocatorBuffer;
RWStructuredBuffer<FScalarGridData> RWExtinctionGridBuffer;
RWStructuredBuffer<FVectorGridData> RWEmissionGridBuffer;
RWStructuredBuffer<FVectorGridData> RWScatteringGridBuffer;

int FixedBottomLevelResolution;

// Hash index to voxel index
//RWBuffer<uint> RWHashToVoxelBuffer;

// Sampling mode
int bSampleAtVertices;

float HomogeneousThreshold;

groupshared float3 GSExtinctionSum[THREADGROUP_SIZE_1D];
groupshared float GSExtinctionVariance[THREADGROUP_SIZE_1D];
groupshared float GSZeroCrossings[THREADGROUP_SIZE_1D];
groupshared int3 GSAllocatedVoxelResolution;
groupshared int GSAllocatedVoxelCount;

float GetZeroThreshold()
{
	return 1.0e-6;
}

float CalcSampleVariance(float3 MeanValue, float3 SampleValue)
{
	float3 SampleVariance = MeanValue - SampleValue;
	return dot(SampleVariance, SampleVariance);
}

float CalcZeroCrossing(float3 SampleValue)
{
	float EmptyVoxelThreshold = 1.0e-5;
	return all(SampleValue < EmptyVoxelThreshold) ? 1.0 : 0.0;
}

#if 0
uint2 CalcStencil(uint VoxelCount)
{
	//int x = i % 4;
	//int y = i / 4;
	//int z = i % 16;
	uint2 Stencil = 0;
	
	float ZeroThreshold = GetZeroThreshold();
	for (int i = 0; i < min(VoxelCount, 32); ++i)
	{
		int bit = i;
		Stencil[0] |= any(GSExtinction[i] > ZeroThreshold) ? 1 << bit : 0;
	}
	
	for (i = 32; i < min(VoxelCount, 64); ++i)
	{
		int bit = i - 32;
		Stencil[1] |= any(GSExtinction[i] > ZeroThreshold) ? 1 << bit : 0;
	}

	return Stencil;
}
#endif

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RasterizeBottomLevelOrthoGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID
)
{
	ResolvedView = ResolveView();

	uint RasterTileIndex = GetUnWrappedDispatchGroupId(GroupId);
	if (RasterTileIndex >= RasterTileAllocatorBuffer[0])
	{
		return;
	}

	uint TopLevelGridLinearIndex = RasterTileBuffer[RasterTileIndex].TopLevelGridLinearIndex;
	uint3 TopLevelGridVoxelIndex = GetVoxelIndex(TopLevelGridLinearIndex, TopLevelGridResolution);

	float3 TopLevelGridVoxelWorldBoundsMin;
	float3 TopLevelGridVoxelWorldBoundsMax;
	CalcVoxelBounds(TopLevelGridVoxelIndex, TopLevelGridVoxelWorldBoundsMin, TopLevelGridVoxelWorldBoundsMax);
	float3 TopLevelGridVoxelWorldBoundsExtent = TopLevelGridVoxelWorldBoundsMax - TopLevelGridVoxelWorldBoundsMin;

	// Note: Raster tiles map to non-empty voxels by definition
	if (PrimitiveIntersectsVoxel(TopLevelGridVoxelWorldBoundsMin, TopLevelGridVoxelWorldBoundsMax))
	{
		// Map thread id to voxel-space position
		uint BottomLevelGridLinearOffset = RasterTileBuffer[RasterTileIndex].BottomLevelGridLinearOffset;

		FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[TopLevelGridLinearIndex];
		int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

	#if DIM_ENABLE_INDIRECTION_GRID
		int3 BottomLevelVoxelIndex = MortonDecode3(BottomLevelGridLinearOffset) * THREADGROUP_SIZE_3D + GroupThreadId;
		BottomLevelVoxelResolution *= FixedBottomLevelResolution;
	#else
		int3 BottomLevelVoxelIndex = MortonDecode3(BottomLevelGridLinearOffset) * THREADGROUP_SIZE_3D + GroupThreadId;
	#endif

		// Unwrap the 3D index as a 2D index..
		int2 BottomLevelVoxelIndexAs2D = int2(BottomLevelVoxelIndex.y * BottomLevelVoxelResolution.x + BottomLevelVoxelIndex.x, BottomLevelVoxelIndex.z);

		float3 Jitter = 0.5;
		if (bJitter)
		{
			Jitter.xy = BlueNoiseVec2(BottomLevelVoxelIndexAs2D, View.StateFrameIndex);
			Jitter.z = BlueNoiseScalar(BottomLevelVoxelIndexAs2D, View.StateFrameIndex);
		}

		float3 Extinction = 0;
		float3 Emission = 0;
		float3 Albedo = 0;
		if (all(BottomLevelVoxelIndex < BottomLevelVoxelResolution))
		{
			float3 UVW;
			if (bSampleAtVertices)
			{
				UVW = BottomLevelVoxelIndex / float3(BottomLevelVoxelResolution - 1);
			}
			else
			{
				UVW = float3(BottomLevelVoxelIndex + Jitter) / float3(BottomLevelVoxelResolution);
			}

			float3 WorldPosition = UVW * TopLevelGridVoxelWorldBoundsExtent + TopLevelGridVoxelWorldBoundsMin;
			float3 LocalPosition = mul(float4(WorldPosition, 1), WorldToLocal).xyz;
			if (LocalPositionIntersectsPrimitive(LocalPosition))
			{
				float3 WorldOrigin = mul(float4(0.0, 0.0, 0.0, 1.0), LocalToWorld).xyz;
				float3 WorldVoxelPos = mul(float4(1.0, 1.0, 1.0, 1.0), LocalToWorld).xyz;
				float WorldVoxelVolume = length(WorldVoxelPos - WorldOrigin);

				// Setup evaluation context
				FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
				MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);
				MaterialParameters.PrimitiveId = PrimitiveId;
				MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
				MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
				// TODO: Add object centroid to LWC.ObjectWorldPosition
				MaterialParameters.LWCData.LocalToWorld = WSPromote(LocalToWorld);
				MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(WorldToLocal);

				// Evaluate material graph
				FPixelMaterialInputs PixelMaterialInputs;
				CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

				// Extract volume rendering coefficients
				Extinction = SampleExtinctionCoefficients(PixelMaterialInputs) / WorldVoxelVolume * SQRT_THREE;
				Emission = SampleEmissive(PixelMaterialInputs) / WorldVoxelVolume * SQRT_THREE;
				Albedo = SampleAlbedo(PixelMaterialInputs) * Extinction;
			}
		}

		// Aggregate in group-shared memory
		uint LinearThreadIndex = MortonEncode3(GroupThreadId);
		GSExtinctionSum[LinearThreadIndex] = Extinction;

		if (all(GroupThreadId == 0))
		{
	#if DIM_ENABLE_INDIRECTION_GRID
			GSAllocatedVoxelResolution = FixedBottomLevelResolution;
	#else
			GSAllocatedVoxelResolution = BottomLevelVoxelResolution;
	#endif
			GSAllocatedVoxelCount = GSAllocatedVoxelResolution.x * GSAllocatedVoxelResolution.y * GSAllocatedVoxelResolution.z;
		}
		GroupMemoryBarrierWithGroupSync();

		// Parallel Sum
		for (int NeighborOffsetSum = 1; NeighborOffsetSum < THREADGROUP_SIZE_1D; NeighborOffsetSum = (NeighborOffsetSum << 1))
		{
			int NeighborIndex = LinearThreadIndex + NeighborOffsetSum;
			GSExtinctionSum[LinearThreadIndex] += (NeighborIndex < THREADGROUP_SIZE_1D) ? GSExtinctionSum[NeighborIndex] : 0.0;

			GroupMemoryBarrierWithGroupSync();
		}

#if DIM_ENABLE_HOMOGENEOUS_AGGREGATION
		// Calculate variance
		float3 ExtinctionMean = GSExtinctionSum[0] * rcp(GSAllocatedVoxelCount);
		GSExtinctionVariance[LinearThreadIndex] = CalcSampleVariance(ExtinctionMean, Extinction);
		GSZeroCrossings[LinearThreadIndex] = CalcZeroCrossing(Extinction);

		// Parallel Sum
		for (int NeighborOffset = 1; NeighborOffset < THREADGROUP_SIZE_1D; NeighborOffset = (NeighborOffset << 1))
		{
			int NeighborIndex = LinearThreadIndex + NeighborOffset;
			GSExtinctionVariance[LinearThreadIndex] += (NeighborIndex < THREADGROUP_SIZE_1D) ? GSExtinctionVariance[NeighborIndex] : 0.0;
			GSZeroCrossings[LinearThreadIndex] += (NeighborIndex < THREADGROUP_SIZE_1D) ? GSZeroCrossings[NeighborIndex] : 0.0;

			GroupMemoryBarrierWithGroupSync();
		}

		float ExtinctionVariance = GSExtinctionVariance[0] * rcp(GSAllocatedVoxelCount - 1);
		float StdRelError = sqrt(ExtinctionVariance * rcp(GSAllocatedVoxelCount)) * rcp(Luminance(ExtinctionMean));

		// Declare low-variance to constitute a region of homogeneity
		if (all(GroupThreadId == 0) && (StdRelError < HomogeneousThreshold) && (GSZeroCrossings[0] < 4))
		{
			Extinction = ExtinctionMean;
			GSAllocatedVoxelCount = 1; 
			GSAllocatedVoxelResolution = 1;
		}
#endif // #if DIM_ENABLE_HOMOGENEOUS_AGGREGATION

	#if DIM_ENABLE_INDIRECTION_GRID
		uint IndirectionIndexOffset = GetBottomLevelIndex(TopLevelGridData);
		uint IndirectionGridLinearOffset = IndirectionIndexOffset + BottomLevelGridLinearOffset;

		FTopLevelGridData IndirectionData = RWIndirectionGridBuffer[IndirectionGridLinearOffset];
		AllMemoryBarrierWithGroupSync();
		bool bWasAlreadyAllocated = IsBottomLevelAllocated(IndirectionData);
	#else
		bool bWasAlreadyAllocated = IsBottomLevelAllocated(TopLevelGridData);
	#endif
		if (all(GroupThreadId == 0) && !bWasAlreadyAllocated)
		{
			uint BottomLevelIndex = EMPTY_VOXEL_INDEX;

			bool bShouldAllocate = any(GSExtinctionSum[0] > GetZeroThreshold());
			if (bShouldAllocate)
			{
				InterlockedAdd(RWBottomLevelGridAllocatorBuffer[0], GSAllocatedVoxelCount, BottomLevelIndex);

				// Guard against over allocation
				if (BottomLevelIndex + GSAllocatedVoxelCount > BottomLevelGridBufferSize)
				{
					// Declare the buffer to be filled
					uint Dummy;
					InterlockedExchange(RWBottomLevelGridAllocatorBuffer[0], BottomLevelGridBufferSize, Dummy);

					BottomLevelIndex = EMPTY_VOXEL_INDEX;
					GSAllocatedVoxelCount = 0;
					GSAllocatedVoxelResolution = 0;
				}

				// Update the index with the properly allocated index
				FTopLevelGridData AllocatedGridData = (FTopLevelGridData)0;
				SetBottomLevelIndex(AllocatedGridData, BottomLevelIndex);
				SetBottomLevelVoxelResolution(AllocatedGridData, GSAllocatedVoxelResolution);

#if DIM_ENABLE_INDIRECTION_GRID
				RWIndirectionGridBuffer[IndirectionGridLinearOffset] = AllocatedGridData;
#else
				RWTopLevelGridBuffer[TopLevelGridLinearIndex] = AllocatedGridData;
#endif
			}
			else
			{
				GSAllocatedVoxelCount = 0; 
				GSAllocatedVoxelResolution = 0;
			}
		}

		AllMemoryBarrierWithGroupSync();

		if (LinearThreadIndex < GSAllocatedVoxelCount)
		{
	#if DIM_ENABLE_INDIRECTION_GRID
			FTopLevelGridData IndirectionData = RWIndirectionGridBuffer[IndirectionGridLinearOffset];

			if (IsBottomLevelAllocated(IndirectionData))
			{
				uint BottomLevelVoxelLinearIndex = GetBottomLevelIndex(IndirectionData) + LinearThreadIndex;
				if (bWasAlreadyAllocated)
				{
					Extinction += GetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex]);
					Emission += GetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex]);
					Albedo += GetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex]);
				}

				SetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex], Extinction);
				SetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex], Emission);
				SetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex], Albedo);
			}
	#else
			FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[TopLevelGridLinearIndex];

			if (IsBottomLevelAllocated(TopLevelGridData))
			{
				uint BottomLevelVoxelLinearIndex = GetBottomLevelIndex(TopLevelGridData) + MortonEncode3(BottomLevelVoxelIndex);

				if (bWasAlreadyAllocated)
				{
					Extinction += GetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex]);
					Emission += GetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex]);
					Albedo += GetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex]);
				}

				SetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex], Extinction);
				SetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex], Emission);
				SetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex], Albedo);
			}
	#endif
		}
	}
}

int3 VoxelDimensions;
float4x4 ViewToWorld;

float TanHalfFOV;
float NearPlaneDepth;
float FarPlaneDepth;

#include "HeterogeneousVolumesFrustumVoxelGridUtils.ush"

groupshared float GSExtinctionSumScalar[THREADGROUP_SIZE_1D];

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RasterizeBottomLevelFrustumGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID
)
{
	ResolvedView = ResolveView();

	uint RasterTileIndex = GetUnWrappedDispatchGroupId(GroupId);
	if (RasterTileIndex >= RasterTileAllocatorBuffer[0])
	{
		return;
	}

	uint TopLevelGridLinearIndex = RasterTileBuffer[RasterTileIndex].TopLevelGridLinearIndex;
	FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[TopLevelGridLinearIndex];
	int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

	// Setup evaluation context
	float3 TopLevelVoxelPos = GetVoxelIndex(TopLevelGridLinearIndex, TopLevelGridResolution);
	float3 BottomLevelVoxelPos = TopLevelVoxelPos * BottomLevelVoxelResolution + GroupThreadId;

	float3 Jitter = 0.5;
	if (bJitter)
	{
		Jitter.z = BlueNoiseScalar(BottomLevelVoxelPos.xy, View.StateFrameIndex);
	}

	float3 LocalVoxelPos = GroupThreadId + Jitter;
	float3 VoxelPosition = TopLevelVoxelPos + LocalVoxelPos / BottomLevelVoxelResolution;

	float3 ViewPos = VoxelToView(VoxelPosition, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);
	float3 WorldPosition = mul(float4(ViewPos, 1), ViewToWorld).xyz;

	float3 Extinction = 0;
	float3 Emission = 0;
	float3 Albedo = 0;

	if (WorldPositionIntersectsPrimitive(WorldPosition))
	{
		float3 WorldOrigin = mul(float4(0.0, 0.0, 0.0, 1.0), LocalToWorld).xyz;
		float3 WorldVoxelPos = mul(float4(1.0, 1.0, 1.0, 1.0), LocalToWorld).xyz;
		float WorldVoxelVolume = length(WorldVoxelPos - WorldOrigin);

		FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
		MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);
		MaterialParameters.PrimitiveId = PrimitiveId;
		MaterialParameters.AbsoluteWorldPosition = DFPromote(WorldPosition);
		MaterialParameters.LWCData.AbsoluteWorldPosition = WSPromote(WorldPosition);
		// TODO: Add object centroid to LWC.ObjectWorldPosition
		MaterialParameters.LWCData.LocalToWorld = WSPromote(LocalToWorld);
		MaterialParameters.LWCData.WorldToLocal = WSPromoteInverse(WorldToLocal);

		// Evaluate material graph
		FPixelMaterialInputs PixelMaterialInputs;
		CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

		// Rasterize coefficients
		Extinction = SampleExtinctionCoefficients(PixelMaterialInputs) / WorldVoxelVolume * SQRT_THREE;
		Emission = SampleEmissive(PixelMaterialInputs);
		Albedo = SampleAlbedo(PixelMaterialInputs) * Extinction;
	}

	// Aggregate in group-shared memory
	uint LinearThreadIndex = MortonEncode3(GroupThreadId);
	GSExtinctionSumScalar[LinearThreadIndex] = Luminance(Extinction);
	if (all(GroupThreadId == 0))
	{
		GSAllocatedVoxelCount = BottomLevelVoxelResolution.x * BottomLevelVoxelResolution.y * BottomLevelVoxelResolution.z;
	}
	GroupMemoryBarrierWithGroupSync();

	// Parallel Sum
	for (int NeighborOffset = 1; NeighborOffset < THREADGROUP_SIZE_1D; NeighborOffset = (NeighborOffset << 1))
	{
		int NeighborIndex = LinearThreadIndex + NeighborOffset;
		GSExtinctionSumScalar[LinearThreadIndex] += (NeighborIndex < THREADGROUP_SIZE_1D) ? GSExtinctionSumScalar[NeighborIndex] : 0.0;

		GroupMemoryBarrierWithGroupSync();
	}

	uint BottomLevelIndex = EMPTY_VOXEL_INDEX;

	// Allocate if non-zero voxel data
	bool bWasAlreadyAllocated = IsBottomLevelAllocated(TopLevelGridData);
	AllMemoryBarrierWithGroupSync();
	if (all(GroupThreadId == 0) && !bWasAlreadyAllocated)
	{
		uint3 AllocatedVoxelResolution = BottomLevelVoxelResolution;
		bool bShouldAllocate = GSExtinctionSumScalar[0] > GetZeroThreshold();
		if (bShouldAllocate)
		{
			GSAllocatedVoxelCount = AllocatedVoxelResolution.x * AllocatedVoxelResolution.y * AllocatedVoxelResolution.z;

			InterlockedAdd(RWBottomLevelGridAllocatorBuffer[0], GSAllocatedVoxelCount, BottomLevelIndex);

			// Guard against over allocation
			if (BottomLevelIndex + GSAllocatedVoxelCount > BottomLevelGridBufferSize)
			{
				// Declare the buffer to be filled
				uint Dummy;
				InterlockedExchange(RWBottomLevelGridAllocatorBuffer[0], BottomLevelGridBufferSize, Dummy);

				BottomLevelIndex = EMPTY_VOXEL_INDEX;
				GSAllocatedVoxelCount = 0; 
				AllocatedVoxelResolution = 0;
			}

			// Update the index with the properly allocated index
			FTopLevelGridData AllocatedGridData = (FTopLevelGridData)0;
			SetBottomLevelIndex(AllocatedGridData, BottomLevelIndex);
			SetBottomLevelVoxelResolution(AllocatedGridData, AllocatedVoxelResolution);
			RWTopLevelGridBuffer[TopLevelGridLinearIndex] = AllocatedGridData;
		}
		else
		{
			GSAllocatedVoxelCount = 0; 
			AllocatedVoxelResolution = 0;
		}
	}

	AllMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < GSAllocatedVoxelCount)
	{
		FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[TopLevelGridLinearIndex];
		if (IsBottomLevelAllocated(TopLevelGridData))
		{
			uint BottomLevelVoxelLinearIndex = GetBottomLevelIndex(TopLevelGridData) + LinearThreadIndex;

			if (bWasAlreadyAllocated)
			{
				Extinction += GetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex]);
				Emission += GetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex]);
				Albedo += GetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex]);
			}

			SetExtinction(RWExtinctionGridBuffer[BottomLevelVoxelLinearIndex], Extinction);
			SetEmission(RWEmissionGridBuffer[BottomLevelVoxelLinearIndex], Emission);
			SetAlbedo(RWScatteringGridBuffer[BottomLevelVoxelLinearIndex], Albedo);
		}
	}
}
