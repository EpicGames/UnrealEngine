// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS 0

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#ifndef DIM_APPLY_SHADOW_TRANSMITTANCE
#define DIM_APPLY_SHADOW_TRANSMITTANCE 0
#endif // DIM_APPLY_SHADOW_TRANSMITTANCE

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#if VIRTUAL_SHADOW_MAP
#include "../VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#include "HeterogeneousVolumesPreshadingUniformBufferUtils.ush"
#include "HeterogeneousVolumesPreshadingUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesLightingUtils.ush"

#include "HeterogeneousVolumesRayMarchingUtils.ush"

int3 TextureResolution;
Texture3D InputTexture;
RWTexture3D<float3> RWOutputTexture;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void CopyTexture3D(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 Index = DispatchThreadId;
	if (all(Index < TextureResolution))
	{
		RWOutputTexture[Index] = InputTexture[Index].xyz;
	}
}

Texture3D InputTextureLHS;
Texture3D InputTextureRHS;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void MaxTexture3D(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 Index = DispatchThreadId;
	if (all(Index < TextureResolution))
	{
		float3 ValueLHS = InputTextureLHS[Index].xyz;
		float3 ValueRHS = InputTextureRHS[Index].xyz;
#if 0
		if (any(ValueRHS > 0.0))
		{
			ValueRHS = 1.0;
		}
#endif
		RWOutputTexture[Index] = max(ValueLHS, ValueRHS);
	}
}

SamplerState TextureSampler;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void GenerateMips3D(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 Index = DispatchThreadId;
	if (all(Index < TextureResolution))
	{
#if 1
		float3 UVW = (Index + 0.5) / float3(TextureResolution);
		RWOutputTexture[Index] = InputTexture.SampleLevel(TextureSampler, UVW, 0).rgb;
#else
		float3 Value = 0;
		UNROLL for (uint i = 0; i < 8; ++i)
		{
			uint3 BaseIndex = Index * 2 + float3((i & 4u) >> 2, (i & 2u) >> 1, (i & 1u));
			BaseIndex = min(BaseIndex, TextureResolution * 2);
			Value += InputTexture[BaseIndex].xyz;
		}
		Value *= 0.125;

		RWOutputTexture[Index] = Value;
#endif
	}
}

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void GenerateMin3D(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 Index = DispatchThreadId;
	if (all(Index < TextureResolution))
	{
		float3 Value = 65535.0f;
		UNROLL for (uint i = 0; i < 8; ++i)
		{
			int3 BaseIndex = Index * 2 + float3((i & 4u) >> 2, (i & 2u) >> 1, (i & 1u));
			BaseIndex = min(BaseIndex, TextureResolution * 2);
			Value = min(Value, Luminance(InputTexture[BaseIndex].xyz));
		}

		RWOutputTexture[Index] = Value;
	}
}

RWTexture3D<float3> RWLightingCacheTexture;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void RenderLightingCacheWithPreshadingCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	float3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= LightingCacheResolution))
	{
		return;
	}
	float3 VoxelJitter = 0.5;
	if (bJitter)
	{
		uint3 Rand32Bits = Rand4DPCG32(uint4(VoxelIndex, View.StateFrameIndexMod8)).xyz;
		VoxelJitter = float3(Rand32Bits) / float(uint(0xffffffff));
	}
	//float3 UVW = (VoxelIndex + VoxelJitter) / float3(GetLightingCacheResolution());
	float3 UVW = (VoxelIndex + 0.5) / float3(GetLightingCacheResolution());

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float3 LocalRayOrigin = UVW * GetLocalBoundsExtent() * 2.0 + LocalBoundsMin;
	float3 WorldRayOrigin = mul(float4(LocalRayOrigin, 1.0), GetLocalToWorld()).xyz;

	// Existence culling
	float MipLevel = 0.0f;
	FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalRayOrigin, WorldRayOrigin, MipLevel);
	float3 Extinction = SampleExtinction(SampleContext);
	float Epsilon = 1.0e-7;
	if (all(Extinction < Epsilon))
	{
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
		RWLightingCacheTexture[VoxelIndex] = 1.0f;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
		RWLightingCacheTexture[VoxelIndex] = 0.0f;
#endif // DIM_LIGHTING_CACHE_MODE
		return;
	}

	// Build shadow ray
	const FDeferredLightData LightData = InitDeferredLightFromUniforms(GetLightType(), GetVolumetricScatteringIntensity());
#if DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_TRANSMITTANCE
	float3 L = LightData.Direction;
	float3 ToLight = L * GetMaxShadowTraceDistance();
	if (GetLightType() != LIGHT_TYPE_DIRECTIONAL)
	{
		// Note: this function also permutes ToLight and L
		float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldRayOrigin, PrimaryView.PreViewTranslation);
		GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
	
		if (LightData.bRectLight)
		{
			FRect Rect = GetRect(ToLight, LightData);
		}
		else
		{
			FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
		}
	}

	float3 Transmittance = ComputeTransmittance(WorldRayOrigin, ToLight, GetMaxStepCount());
	RWLightingCacheTexture[VoxelIndex] = Transmittance;

#elif DIM_LIGHTING_CACHE_MODE == LIGHTING_CACHE_INSCATTERING
	float3 Inscattering = ComputeInscattering(WorldRayOrigin, LightData, GetLightType(), GetMaxStepCount(), CalcShadowBias(), ShouldApplyShadowTransmittance());
	RWLightingCacheTexture[VoxelIndex] += Inscattering * View.PreExposure;
#endif // DIM_LIGHTING_CACHE_MODE
}

RWTexture2D<float4> RWLightingTexture;
RWTexture2D<float4> RWVelocityTexture;

int2 GetDownsampledResolution(int2 Resolution, int Factor)
{
	return (Resolution + Factor - 1) / Factor;
}

int2 GetScaledViewRect()
{
	return GetDownsampledResolution(View.ViewSizeAndInvSize.xy, GetDownsampleFactor());
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderSingleScatteringWithPreshadingCS(
	uint GroupId : SV_GroupID,
	uint GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	// Create screen ray
	if (any(DispatchThreadId.xy >= GetScaledViewRect()))
	{
		return;
	}

	uint LinearIndex = DispatchThreadId.y * GetGroupCount().x * THREADGROUP_SIZE_2D + DispatchThreadId.x;
	uint2 ViewPixelCoord = DispatchThreadId.xy * DownsampleFactor + View.ViewRectMin.xy;
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy / DownsampleFactor;

	float3 Radiance = 0.0;
	float4 LightingTexture = RWLightingTexture[PixelCoord];
	float3 Transmittance = LightingTexture.aaa;
	float PrevOpacity = Luminance(1.0 - Transmittance);

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), GetMaxTraceDistance());
	DeviceZ = ConvertToDeviceZ(SceneDepth);

	float Jitter = ShouldJitter() ? InterleavedGradientNoise(ViewPixelCoord, View.StateFrameIndexMod8) : 0.0;

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(GetTranslatedWorldCameraPosFromView(ViewPixelCoord + View.TemporalAAJitter.xy), PrimaryView.PreViewTranslation));
	float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(ViewPixelCoord + View.TemporalAAJitter.xy, DeviceZ, 1)));
	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection);
	WorldRayDirection /= WorldRayLength;

	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), GetWorldToLocal()).xyz;
	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();

	// Test bounding volume
	float2 HitT = IntersectAABB(LocalRayOrigin, LocalRayDirection, 0.0, LocalRayLength, LocalBoundsMin, LocalBoundsMax);
	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		FRayMarchingContext RayMarchingContext = CreateRayMarchingContext(
			// Local-space
			LocalRayOrigin,
			LocalRayDirection,
			HitT.x,
			HitT.y,
			// World-space
			WorldRayOrigin,
			WorldRayDirection,
			// Ray-step attributes
			Jitter,
			CalcStepSize(LocalRayDirection),
			GetMaxStepCount(),
			ShouldApplyEmissionAndTransmittance(),
			ShouldApplyDirectLighting(),
			DIM_APPLY_SHADOW_TRANSMITTANCE
		);

		const FDeferredLightData LightData = InitDeferredLightFromUniforms(GetLightType(), GetVolumetricScatteringIntensity());
		uint StepCount = CalcStepCount(RayMarchingContext);

		float ScatterHitT = HitT.y;
		RayMarchSingleScattering(
			RayMarchingContext,
			LightData,
			GetLightType(),
			StepCount,
			ScatterHitT,
			Radiance,
			Transmittance
		);

		// Radiance
		LightingTexture.rgb += Radiance * View.PreExposure;
		if (ShouldApplyEmissionAndTransmittance())
		{
			float Opacity = Luminance(1.0 - Transmittance);
			LightingTexture.a = Luminance(Transmittance);

			if (ShouldWriteVelocity())
			{
				bool bWriteVelocity =
					(ScatterHitT < HitT.y) &&
					(PrevOpacity < 0.5) &&
					(Opacity > 0.5);

				if (bWriteVelocity)
				{
					float WorldScatterHitT = ScatterHitT * RayMarchingContext.LocalToWorldScale;
					float3 Velocity = Calculate3DVelocityAtWorldPos(WorldRayOrigin, WorldRayDirection, WorldScatterHitT);
					RWVelocityTexture[PixelCoord] = EncodeVelocityToTexture(Velocity);
				}
			}
		}

		RWLightingTexture[PixelCoord] = LightingTexture;
	}
}
