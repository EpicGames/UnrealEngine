// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesStochasticFiltering.ush"

struct FVolumeSampleContext
{
	float3 UVW;
	float MipLevel;
};

float4x4 GetLocalToWorld()
{
	return PreshadingUniformBuffer.LocalToWorld;
}

float4x4 GetWorldToLocal()
{
	return PreshadingUniformBuffer.WorldToLocal;
}

float3 GetLocalBoundsOrigin()
{
	return PreshadingUniformBuffer.LocalBoundsOrigin;
}

float3 GetLocalBoundsExtent()
{
	return PreshadingUniformBuffer.LocalBoundsExtent;
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	float3 Extinction = Texture3DSampleLevel(
		PreshadingUniformBuffer.ExtinctionTexture,
		PreshadingUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).rgb;

	return Extinction;
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	float3 Emission = Texture3DSampleLevel(
		PreshadingUniformBuffer.EmissionTexture,
		PreshadingUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).xyz;

	return Emission;
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = Texture3DSampleLevel(
		PreshadingUniformBuffer.AlbedoTexture,
		PreshadingUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).xyz;

	return Albedo;
}

FVolumeSampleContext CreateVolumeSampleContext(
	float3 LocalPosition,
	float3 WorldPosition,
	float3 WorldPosition_DDX,
	float3 WorldPosition_DDY,
	float MipLevel
)
{
	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();

	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.UVW = (LocalPosition - LocalBoundsMin) / (LocalBoundsMax - LocalBoundsMin);
	VolumeSampleContext.MipLevel = MipLevel;

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0.0;
	float3 WorldPosition_DDY = 0.0;

	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int SamplingMethod)
{
	// TODO: Use FilterWidth determine MipLevel
	//float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (SamplingMethod == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (SamplingMethod == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (SamplingMethod == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;
	return CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
}