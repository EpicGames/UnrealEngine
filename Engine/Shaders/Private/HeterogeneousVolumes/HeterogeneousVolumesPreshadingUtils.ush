// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesStochasticFiltering.ush"

int bApplyEmissionAndTransmittance;
bool ShouldApplyEmissionAndTransmittance()
{
	return bApplyEmissionAndTransmittance != 0;
}

int bApplyDirectLighting;
bool ShouldApplyDirectLighting()
{
	return bApplyDirectLighting != 0;
}

int LightType;
int GetLightType()
{
	return LightType;
}

uint VirtualShadowMapId;
uint GetVirtualShadowMapId()
{
	return VirtualShadowMapId;
}

int MipLevel;
int GetMipLevel()
{
	return MipLevel;
}

int MaxStepCount;
int GetMaxStepCount()
{
	return MaxStepCount;
}

int bJitter;
bool ShouldJitter()
{
	return bJitter;
}

float VolumetricScatteringIntensity;
float GetVolumetricScatteringIntensity()
{
	return VolumetricScatteringIntensity;
}

float MaxStepSize;
float GetMaxStepSize()
{
	return MaxStepSize;
}

int3 GroupCount;
int3 GetGroupCount()
{
	return GroupCount;
}

int DownsampleFactor;
int GetDownsampleFactor()
{
	return DownsampleFactor;
}

float MaxTraceDistance;
float GetMaxTraceDistance()
{
	return MaxTraceDistance;
}

float MaxShadowTraceDistance;
float GetMaxShadowTraceDistance()
{
	return MaxShadowTraceDistance;
}

float StepSize;
float GetStepSize()
{
	return StepSize;
}

// TEST..
float StepFactor;
float GetStepFactor()
{
	//float StepFactor = 1.0;
	return StepFactor;
}

float ShadowStepSize;
float GetShadowStepSize()
{
	//float ShadowStepSize = 1.0;
	return ShadowStepSize;
}

float ShadowStepFactor;
float GetShadowStepFactor()
{
	//float ShadowStepFactor = 1.0;
	return ShadowStepFactor;
}
// END TEST..

int GetStochasticFilteringMode()
{
	return STOCHASTIC_FILTERING_TRILINEAR;
}

int ShouldApplyShadowTransmittance()
{
	int bShouldApplyShadowTransmittance = 1;
	return bShouldApplyShadowTransmittance;
}

//int bApplyHeightFog;
int ShouldApplyHeightFog()
{
	int bApplyHeightFog = 1;
	return bApplyHeightFog;
}

//int bApplyVolumetricFog;
int ShouldApplyVolumetricFog()
{
	int bApplyVolumetricFog = 1;
	return bApplyVolumetricFog;
}

//float IndirectInscatteringFactor;
float GetIndirectInscatteringFactor()
{
	float IndirectInscatteringFactor = 1.0;
	return IndirectInscatteringFactor;
}

// TODO: Implement ambient occlusion for preshading pipeline
float EvalAmbientOcclusion(float3 WorldPosition)
{
	return 1.0;
}

// Cinematic feature options
bool UseInscatteringVolume()
{
	bool bUseInscatteringVolume = true;
	return bUseInscatteringVolume;
}

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
	return INDIRECT_LIGHTING_MODE_OFF;
}

bool ShouldWriteVelocity()
{
	bool bShouldWriteVelocity = false;
	return bShouldWriteVelocity;
}

bool IsOfflineRender()
{
	bool bIsOfflineRender = false;
	return bIsOfflineRender;
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
	return FOG_INSCATTERING_MODE_LINEAR_APPROX;
}

uint3 GetVolumeResolution()
{
	return PreshadingUniformBuffer.VolumeResolution;
}
