// Copyright Epic Games, Inc. All Rights Reserved.

#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"

struct FRayMarchingContext
{
	// Local-space
	float3 LocalRayOrigin;
	float3 LocalRayDirection;
	float LocalRayTMin;
	float LocalRayTMax;

	// World-space
	float3 WorldRayOrigin;
	float3 WorldRayDirection;
#if USE_ANALYTIC_DERIVATIVES
	float3 WorldRayDirection_DDX;
	float3 WorldRayDirection_DDY;
#endif // USE_ANALYTIC_DERIVATIVES
	float LocalToWorldScale;

	// Ray-step attributes
	float Jitter;
	float StepSize;
	uint MaxStepCount;

	float MipLevel;

	// Lighting
	int bApplyEmissionAndTransmittance;
	int bApplyDirectLighting;
	int bApplyShadowTransmittance;

	// TEMPORARY STORAGE!!
	FRandomSequence RandSequence;
	float LocalScatterT0;
	float LocalScatterT1;
	float3 Transmittance0;
};

FRayMarchingContext CreateRayMarchingContext(
	// Local-space
	float3 LocalRayOrigin,
	float3 LocalRayDirection,
	float3 LocalRayDirection_DDX,
	float3 LocalRayDirection_DDY,
	float LocalRayTMin,
	float LocalRayTMax,
	// World-space
	float3 WorldRayOrigin,
	float3 WorldRayDirection,
	float3 WorldRayDirection_DDX,
	float3 WorldRayDirection_DDY,
	// Ray-step attributes
	float Jitter,
	float StepSize,
	uint MaxStepCount,
	// Lighting
	int bApplyEmissionAndTransmittance,
	int bApplyDirectLighting,
	int bApplyShadowTransmittance
)
{
	FRayMarchingContext Context = (FRayMarchingContext)0;

	// Local-space
	Context.LocalRayOrigin = LocalRayOrigin;
	Context.LocalRayDirection = LocalRayDirection;
	Context.LocalRayTMin = LocalRayTMin;
	Context.LocalRayTMax = LocalRayTMax;
	// World-space
	Context.WorldRayOrigin = WorldRayOrigin;
	Context.WorldRayDirection = WorldRayDirection;
#if USE_ANALYTIC_DERIVATIVES
	Context.WorldRayDirection_DDX = WorldRayDirection_DDX;
	Context.WorldRayDirection_DDY = WorldRayDirection_DDY;
#endif // USE_ANALYTIC_DERIVATIVES
	// Ray-step attributes
	Context.Jitter = Jitter;
	Context.StepSize = StepSize;
	Context.MaxStepCount = MaxStepCount;
	// Lighting
	Context.bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
	Context.bApplyDirectLighting = bApplyDirectLighting;
	Context.bApplyShadowTransmittance = bApplyShadowTransmittance;

	// Transform local-scale properties into world-scale
	float3 LocalRayHit = LocalRayOrigin + LocalRayDirection * LocalRayTMax;
	float3 WorldRayHit = mul(float4(LocalRayHit, 1.0), GetLocalToWorld()).xyz;
	float WorldRayTMax = length(WorldRayHit - WorldRayOrigin);
	Context.LocalToWorldScale = WorldRayTMax / LocalRayTMax;

	Context.MipLevel = 0.0f;

	return Context;
}

FRayMarchingContext CreateRayMarchingContext(
	// Local-space
	float3 LocalRayOrigin,
	float3 LocalRayDirection,
	float LocalRayTMin,
	float LocalRayTMax,
	// World-space
	float3 WorldRayOrigin,
	float3 WorldRayDirection,
	// Ray-step attributes
	float Jitter,
	float StepSize,
	uint MaxStepCount,
	// Lighting
	int bApplyEmissionAndTransmittance,
	int bApplyDirectLighting,
	int bApplyShadowTransmittance
)
{
	float3 LocalRayDirection_DDX = 0.0;
	float3 LocalRayDirection_DDY = 0.0;
	float3 WorldRayDirection_DDX = 0.0;
	float3 WorldRayDirection_DDY = 0.0;

	return CreateRayMarchingContext(
		// Local-space
		LocalRayOrigin,
		LocalRayDirection,
		LocalRayDirection_DDX,
		LocalRayDirection_DDY,
		LocalRayTMin,
		LocalRayTMax,
		// World-space
		WorldRayOrigin,
		WorldRayDirection,
		WorldRayDirection_DDX,
		WorldRayDirection_DDY,
		// Ray-step attributes
		Jitter,
		StepSize,
		MaxStepCount,
		// Lighting
		bApplyEmissionAndTransmittance,
		bApplyDirectLighting,
		bApplyShadowTransmittance
	);
}

float CalcStepSize(float3 LocalRayDirection, float LocalStepSize, float LocalStepFactor)
{
	if (LocalStepSize <= 0.0f)
	{
		// Intersect aligned voxel to determine candidate t-values and use the minimum distance
		float3 VoxelSize = GetLocalBoundsExtent() / float3(GetVolumeResolution());
		float3 tValues = abs(VoxelSize / LocalRayDirection);
		float VoxelStepSize = min(tValues.x, min(tValues.y, tValues.z));
		LocalStepSize = VoxelStepSize * LocalStepFactor;
	}
	return max(LocalStepSize, 0.01f);
}

float CalcStepSize(float3 LocalRayDirection)
{
	float LocalStepSize = GetStepSize();
	float LocalStepFactor = GetStepFactor();
	return CalcStepSize(LocalRayDirection, LocalStepSize, LocalStepFactor);
}

float CalcShadowStepSize(float3 LocalRayDirection)
{
	float LocalStepSize = GetShadowStepSize();
	float LocalStepFactor = GetShadowStepFactor();
	return CalcStepSize(LocalRayDirection, LocalStepSize, LocalStepFactor);
}

float RecalculateStepSize(inout FRayMarchingContext Context)
{
	// Recalculate step-size if the bounding-box diagonal is greater than the maximum step-count
	float DiagonalLength = length(2.0 * GetLocalBoundsExtent());
	uint MaxStepCount = max(DiagonalLength / Context.StepSize, 1);

	float StepSize = (MaxStepCount > Context.MaxStepCount) ? DiagonalLength / Context.MaxStepCount : Context.StepSize;
	return StepSize;
}

uint CalcStepCount(inout FRayMarchingContext Context)
{
	Context.StepSize = RecalculateStepSize(Context);

	float HitSpan = length(Context.LocalRayTMax - Context.LocalRayTMin);
	uint StepCount = max(ceil(HitSpan / Context.StepSize), 1);
	return StepCount;
}