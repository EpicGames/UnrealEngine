// Copyright Epic Games, Inc. All Rights Reserved.

// Change this to force shader recompilation
#pragma message("UESHADERMETADATA_VERSION 498251B5-BC1F-4250-A0A2-7BB218050A9A")

#include "HeterogeneousVolumesRayMarchingTypes.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesTransmittanceVolumeUtils.ush"

#include "../ForwardShadowingCommon.ush"
#include "../VolumeLightingCommon.ush"
#include "../LightData.ush"
#include "../DeferredLightingCommon.ush"

#include "../SHCommon.ush"
#define RadianceCacheInterpolation LumenGIVolumeStruct
#include "../Lumen/LumenTranslucencyVolumeShared.ush"

#ifndef CUSTOM_TRANSMITTANCE_FUNCTION
#define CUSTOM_TRANSMITTANCE_FUNCTION 0
#endif // CUSTOM_TRANSMITTANCE_FUNCTION

#ifndef HARDWARE_RAY_TRACING
#define HARDWARE_RAY_TRACING 0
#endif // HARDWARE_RAY_TRACING

#ifndef HARD_SURFACE_SHADOWING
#define HARD_SURFACE_SHADOWING 1
#endif // HARD_SURFACE_SHADOWING

#ifndef DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP 0
#endif // DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#if DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapSampler.ush"
#endif // DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#include "../HeightFogCommon.ush"

float3 GetWorldVoxelSizeOfLightingCache()
{
	float3 WorldVoxelSize = 0;
	float3 LightingCacheResolutionAsFloat3 = GetLightingCacheResolution();
	if (all(LightingCacheResolutionAsFloat3 > 0))
	{
		float3 VolumeBounds = 2.0f * GetLocalBoundsExtent();
		float3 VoxelSize = VolumeBounds * rcp(LightingCacheResolutionAsFloat3);
		WorldVoxelSize = mul(float4(VoxelSize, 0), GetLocalToWorld()).xyz;
	}

	return WorldVoxelSize;
}

float CalcShadowBias()
{
	float WorldShadowBias = 0.0f;
	float3 LightingCacheResolutionAsFloat3 = GetLightingCacheResolution();
	if (all(LightingCacheResolutionAsFloat3 > 0))
	{
		float3 VolumeBounds = 2.0f * GetLocalBoundsExtent();
		float3 VoxelSize = VolumeBounds * rcp(LightingCacheResolutionAsFloat3);
		float3 WorldVoxelSize = mul(float4(VoxelSize, 0), GetLocalToWorld()).xyz;
		float VoxelDiagonal = length(WorldVoxelSize);
		WorldShadowBias = VoxelDiagonal * GetLightingCacheVoxelBias();
	}
	return WorldShadowBias;
}

float ComputeHardSurfaceShadowFactor(
	float3 TranslatedWorldPosition,
	FDeferredLightData LightData,
	uint LightType
)
{
	float HardSurfaceShadowFactor = 1.0;

#if HARD_SURFACE_SHADOWING
	// Evaluate hard-surface shadow term
	if (LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		float SceneDepth = dot(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin, View.ViewForward);
		bool bShadowingFromValidUVArea = false;
		float ShadowFactor = ComputeDirectionalLightDynamicShadowing(TranslatedWorldPosition, SceneDepth, bShadowingFromValidUVArea);

		if (bShadowingFromValidUVArea)
		{
			HardSurfaceShadowFactor *= ShadowFactor;
		}
	}
	else // Local lights
	{
		bool bShadowingFromValidUVArea = false;
		float ShadowFactor = ComputeVolumeShadowing(TranslatedWorldPosition, LightData.bRadialLight && !LightData.bSpotLight, LightData.bSpotLight, bShadowingFromValidUVArea);

		if (bShadowingFromValidUVArea)
		{
			HardSurfaceShadowFactor *= ShadowFactor;
		}
	}

#if VIRTUAL_SHADOW_MAP
	if (GetVirtualShadowMapId() != INDEX_NONE)
	{
		FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapTranslatedWorld(GetVirtualShadowMapId(), TranslatedWorldPosition);
		HardSurfaceShadowFactor *= VirtualShadowMapSample.ShadowFactor;
	}
#endif // VIRTUALSHADOW_MAP

#endif // HARD_SURFACE_SHADOWING

	return HardSurfaceShadowFactor;
}

void RayMarchTransmittance(
	inout FRayMarchingContext RayMarchingContext,
	uint StepCount,
	inout float3 Transmittance
)
{
	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		float LocalHitT = RayMarchingContext.LocalRayTMin + RayMarchingContext.StepSize * (RayMarchingContext.Jitter + StepIndex);
		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * LocalHitT * RayMarchingContext.LocalToWorldScale;
#if USE_ANALYTIC_DERIVATIVES
		float3 WorldPosition_DDX = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDX * LocalHitT * RayMarchingContext.LocalToWorldScale;
		float3 WorldPosition_DDY = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDY * LocalHitT * RayMarchingContext.LocalToWorldScale;
		WorldPosition_DDX -= WorldPosition;
		WorldPosition_DDY -= WorldPosition;
#else
		float3 WorldPosition_DDX = 0.0;
		float3 WorldPosition_DDY = 0.0;
#endif // USE_ANALYTIC_DERIVATIVES

		FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, WorldPosition_DDX, WorldPosition_DDY, RayMarchingContext.MipLevel);
		float3 Extinction = SampleExtinction(SampleContext);
		Transmittance *= exp(-Extinction * RayMarchingContext.StepSize);

		float Epsilon = 1.0e-7;
		if (all(Transmittance < Epsilon))
		{
			Transmittance = 0.0;
			break;
		}
	}
}

float3 ComputeTransmittance(
	inout FRayMarchingContext RayMarchingContext
)
{
#if CUSTOM_TRANSMITTANCE_FUNCTION
	float WorldRayTMin = RayMarchingContext.LocalRayTMin * RayMarchingContext.LocalToWorldScale;
	float WorldRayTMax = RayMarchingContext.LocalRayTMax * RayMarchingContext.LocalToWorldScale;
	return TransmittanceFunction(RayMarchingContext.WorldRayOrigin, RayMarchingContext.WorldRayDirection, WorldRayTMin, WorldRayTMax);
#elif HARDWARE_RAY_TRACING
	//  TODO: Incorporpate ray jitter
	return ComputeTransmittanceHardwareRayTracing(RayMarchingContext.WorldRayOrigin, RayMarchingContext.WorldRayDirection, RayMarchingContext.LocalRayTMin, RayMarchingContext.LocalRayTMax);
#else // HARDWARE_RAY_TRACING

	float3 Transmittance = 1.0;

	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();
	float2 HitT = IntersectAABB(RayMarchingContext.LocalRayOrigin, RayMarchingContext.LocalRayDirection, RayMarchingContext.LocalRayTMin, RayMarchingContext.LocalRayTMax,
		LocalBoundsMin, LocalBoundsMax);

	float HitSpan = HitT.y - HitT.x;
	if (HitSpan > 0.0)
	{
		RayMarchingContext.LocalRayTMin = HitT.x;
		RayMarchingContext.LocalRayTMax = HitT.y;
		uint StepCount = CalcStepCount(RayMarchingContext);

		RayMarchTransmittance(RayMarchingContext, StepCount, Transmittance);
	}

	return Transmittance;
#endif // HARDWARE_RAY_TRACING
}

float3 ComputeTransmittance(
	float3 WorldRayOrigin,
	float3 ToLight,
	uint MaxStepCount
)
{
	float3 Transmittance = 1.0;

#if DIM_USE_TRANSMITTANCE_VOLUME
	float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 TransmittanceUVW = saturate((LocalRayOrigin - LocalBoundsMin) / (2.0 * GetLocalBoundsExtent()));
	float MipLevel = 0;

	Transmittance = SampleLightingCache(TransmittanceUVW, MipLevel);

#else
	#if DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP
		if (!AVSM.bIsEmpty)
		{
			float3 TranslatedWorldPosition = DFFastAddDemote(WorldRayOrigin, PrimaryView.PreViewTranslation);
			float3 LightTranslatedWorldPosition = TranslatedWorldPosition + ToLight;
			Transmittance = AVSM_SampleTransmittance(TranslatedWorldPosition, LightTranslatedWorldPosition);
		}
		else if (!BeerShadowMap.bIsEmpty)
		{
			float3 TranslatedWorldPosition = DFFastAddDemote(WorldRayOrigin, PrimaryView.PreViewTranslation);
			float3 LightTranslatedWorldPosition = TranslatedWorldPosition + ToLight;
			Transmittance = BeerShadowMap_SampleTransmittance(TranslatedWorldPosition, LightTranslatedWorldPosition);
		}
		else
	#endif
		{
			float3 WorldRayEnd = WorldRayOrigin + ToLight;
			float3 WorldRayDirection = normalize(ToLight);

			float3 LocalRayOrigin = mul(float4(WorldRayOrigin, 1.0), GetWorldToLocal()).xyz;
			float3 LocalRayEnd = mul(float4(WorldRayEnd, 1.0), GetWorldToLocal()).xyz;
			float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
			float LocalRayTMin = 0.0;
			float LocalRayTMax = length(LocalRayDirection);
			LocalRayDirection /= LocalRayTMax;

			float ShadowBias = 0.5;
			float ShadowStepSize = CalcShadowStepSize(LocalRayDirection);
			int bApplyEmissionAndTransmittance = 0;
			int bApplyDirectLighting = 0;
			int bApplyShadowTransmittance = 0;

			FRayMarchingContext ShadowRayMarchingContext = CreateRayMarchingContext(
				LocalRayOrigin,
				LocalRayDirection,
				LocalRayTMin,
				LocalRayTMax,
				WorldRayOrigin,
				WorldRayDirection,
				ShadowBias,
				ShadowStepSize,
				MaxStepCount,
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance
				//RayMarchingContext.MaxShadowTraceDistance
			);

			Transmittance = ComputeTransmittance(ShadowRayMarchingContext);
		}
#endif
	return Transmittance;
}

float3 ComputeIndirectInscattering(
	float3 WorldPosition,
	float3 WorldRayDirection
)
{
	FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(DFPromote(WorldPosition), PrimaryView.WorldToClip, true);

	FTwoBandSHVector RotatedHGZonalHarmonic;
	float3 CameraVector = -WorldRayDirection;
	float PhaseG = 0.0;
	RotatedHGZonalHarmonic.V = float4(1.0f, CameraVector.y, CameraVector.z, CameraVector.x) * float4(1.0f, PhaseG, PhaseG, PhaseG);
	float3 IndirectInscattering = max(DotSH(TranslucencyGISH, RotatedHGZonalHarmonic), 0) / PI;
	IndirectInscattering *= GetIndirectInscatteringFactor();

	// Note: Phase is accounted for in spherical hamonic calculation
	return IndirectInscattering * EvalAmbientOcclusion(WorldPosition);
}

float3 ComputeInscattering(
	float3 WorldPosition,
	FDeferredLightData LightData,
	uint LightType,
	uint MaxStepCount,
	float WorldShadowBias,
	bool bApplyShadowTransmittance
)
{
	float3 L = LightData.Direction;
	//float3 ToLight = L * RayMarchingContext.MaxShadowTraceDistance;
	float3 ToLight = L * 10000;

	float LightAttenuation = 1.0;
	float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WorldPosition, PrimaryView.PreViewTranslation);
	if (LightType != LIGHT_TYPE_DIRECTIONAL)
	{
		LightAttenuation = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
		if (LightData.bRectLight)
		{
			FRect Rect = GetRect(ToLight, LightData);
			LightAttenuation *= IntegrateLight(Rect);
		}
		else
		{
			FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
			Capsule.DistBiasSqr = 0;
			LightAttenuation *= IntegrateLight(Capsule, LightData.bInverseSquared);
		}
	}

	float HardSurfaceShadowFactor = ComputeHardSurfaceShadowFactor(TranslatedWorldPosition, LightData, LightType);
	float3 Inscattering = LightData.Color * LightAttenuation * HardSurfaceShadowFactor;
	if (any(Inscattering > 0.0) && bApplyShadowTransmittance)
	{
		float3 BiasedWorldPosition = WorldPosition + L * WorldShadowBias;
		Inscattering *= ComputeTransmittance(BiasedWorldPosition, ToLight, MaxStepCount);
	}

	if (GetIndirectLightingMode() == INDIRECT_LIGHTING_MODE_LIGHTING_CACHE)
	{
		float3 WorldRayDirection = normalize(WorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
		float3 IndirectInscattering = ComputeIndirectInscattering(WorldPosition, WorldRayDirection);
		// Phase is already accounted for, so add a correction factor
		float IsotropicPhaseRcp = 4.0 * PI;
		Inscattering += IndirectInscattering * IsotropicPhaseRcp;
	}

	return Inscattering;
}

void RayMarchEmissionAbsorption(
	inout FRayMarchingContext RayMarchingContext,
	uint StepCount,
	inout float3 Radiance,
	inout float3 Transmittance
)
{
	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		float LocalHitT = RayMarchingContext.LocalRayTMin + RayMarchingContext.StepSize * (RayMarchingContext.Jitter + StepIndex);
		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * LocalHitT * RayMarchingContext.LocalToWorldScale;
#if USE_ANALYTIC_DERIVATIVES
		float3 WorldPosition_DDX = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDX * LocalHitT * RayMarchingContext.LocalToWorldScale;
		float3 WorldPosition_DDY = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDY * LocalHitT * RayMarchingContext.LocalToWorldScale;
		WorldPosition_DDX -= WorldPosition;
		WorldPosition_DDY -= WorldPosition;
#else
		float3 WorldPosition_DDX = 0.0;
		float3 WorldPosition_DDY = 0.0;
#endif // USE_ANALYTIC_DERIVATIVES

		FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, WorldPosition_DDX, WorldPosition_DDY, RayMarchingContext.MipLevel);
		float3 Extinction = SampleExtinction(SampleContext);

		float3 Emission = SampleEmission(SampleContext);
		Radiance += Emission * RayMarchingContext.StepSize * Transmittance;

		// Accumulate transmittance for the next evaluation
		Transmittance *= exp(-Extinction * RayMarchingContext.StepSize);
	}
}

void RayMarchSingleScattering(
	inout FRayMarchingContext RayMarchingContext,
#if USE_CAMERA_AVSM
	inout FAVSM_Sampler4 TransmittanceSampler,
#endif
	FDeferredLightData LightData,
	uint LightType,
	uint StepCount,
	inout float LocalScatterTMin,
	inout float3 Radiance,
	inout float3 Transmittance
)
{
	LocalScatterTMin = RayMarchingContext.LocalRayTMax;

	float3 FogRadiance = 0;
	float3 PrevTransmittance = Transmittance;
	float4 PrevCombinedFog = 0;
	float WorldFogHitT[] = {
		RayMarchingContext.LocalRayTMin * RayMarchingContext.LocalToWorldScale,
		RayMarchingContext.LocalRayTMax * RayMarchingContext.LocalToWorldScale
	};

	float4 FogHitRadiance[] = {
		float4(0.0, 0.0, 0.0, 0.0),
		float4(0.0, 0.0, 0.0, 0.0)
	};

	if (GetFogInscatteringMode() == FOG_INSCATTERING_MODE_LINEAR_APPROX)
	{
		UNROLL
		for (int i = 0; i < 2; ++i)
		{
			float3 CameraRelative_WorldPosition = RayMarchingContext.WorldRayDirection * WorldFogHitT[i];
			float4 HeightFog = float4(0, 0, 0, 1);
			if (ShouldApplyHeightFog())
			{
				HeightFog = CalculateHeightFog(CameraRelative_WorldPosition);
			}

			float4 CombinedFog = HeightFog;
			if (ShouldApplyVolumetricFog() && FogStruct.ApplyVolumetricFog > 0)
			{
				float3 WorldPosition = CameraRelative_WorldPosition + RayMarchingContext.WorldRayOrigin;
				const uint EyeIndex = 0;
				float3 VolumeUV = ComputeVolumeUV_DEPRECATED(WorldPosition, DFHackToFloat(PrimaryView.WorldToClip));
				CombinedFog = CombineVolumetricFog(HeightFog, VolumeUV, EyeIndex, WorldFogHitT[i]);
			}

			FogHitRadiance[i] = CombinedFog;
		}
	}

	float WorldShadowBias = CalcShadowBias();
	//float WorldShadowBias = View.GeneralPurposeTweak;
	float StepSize = RayMarchingContext.StepSize;
	float JitterSize = StepSize * RayMarchingContext.Jitter;
	// Reuse jitter as random sample
	float RandomSample = RayMarchingContext.Jitter;

	bool bFirstScatterEvent = true;

	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		// Correct the final step size
		float LocalHitT = RayMarchingContext.LocalRayTMin + RayMarchingContext.StepSize * (RayMarchingContext.Jitter + StepIndex);
		if (StepIndex == StepCount - 1)
		{
			float UnjitteredLocalHitT = LocalHitT - JitterSize;
			StepSize = RayMarchingContext.LocalRayTMax - UnjitteredLocalHitT;
		}

		float WorldHitT = LocalHitT * RayMarchingContext.LocalToWorldScale;
		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * WorldHitT;

		FVolumeSampleContext SampleContext;
		if (IsOfflineRender())
		{
			float3 FilterWidth = 1.0;
			float Rand = RayMarchingContext.RandSequence.Get1D();
			int StochasticFilteringMode = GetStochasticFilteringMode();
			SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, FilterWidth, RayMarchingContext.MipLevel, Rand, StochasticFilteringMode);
		}
		else
		{
#if USE_ANALYTIC_DERIVATIVES
			float3 WorldPosition_DDX = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDX * WorldHitT;
			float3 WorldPosition_DDY = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDY * WorldHitT;
			WorldPosition_DDX -= WorldPosition;
			WorldPosition_DDY -= WorldPosition;
#else
			float3 WorldPosition_DDX = 0.0;
			float3 WorldPosition_DDY = 0.0;
#endif // USE_ANALYTIC_DERIVATIVES

			SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, WorldPosition_DDX, WorldPosition_DDY, RayMarchingContext.MipLevel);
		}

		if (RayMarchingContext.bApplyEmissionAndTransmittance)
		{
			float3 Emission = SampleEmission(SampleContext);
			Radiance += Emission * StepSize * Transmittance;

			if (GetIndirectLightingMode() == INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING)
			{
				float3 IndirectInscattering = ComputeIndirectInscattering(WorldPosition, RayMarchingContext.WorldRayDirection);
				float3 Extinction = SampleExtinction(SampleContext);
				float3 Albedo = SampleAlbedo(SampleContext);
				float3 ScatteringCoefficient = Albedo * Extinction;

				Radiance += IndirectInscattering * ScatteringCoefficient * Transmittance * StepSize;
			}
		}

		float3 Extinction = SampleExtinction(SampleContext);
		if (RayMarchingContext.bApplyDirectLighting && (any(Extinction > 0)))
		{
			float3 RadianceSample = 0;

			// Evaluate in-scattering
			float3 Albedo = SampleAlbedo(SampleContext);
			if (any(Albedo > 0.0))
			{
#if DIM_USE_INSCATTERING_VOLUME
				float3 LocalShadowRayOrigin = mul(float4(WorldPosition, 1.0), GetWorldToLocal()).xyz;
				float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
				float3 TransmittanceUVW = saturate((LocalShadowRayOrigin - LocalBoundsMin) / (2.0 * GetLocalBoundsExtent()));
				float MipLevel = 0;

				float3 Inscattering = SampleLightingCache(TransmittanceUVW, MipLevel) * View.OneOverPreExposure;
#else
				float3 Inscattering = ComputeInscattering(WorldPosition, LightData, LightType, RayMarchingContext.MaxStepCount, WorldShadowBias, RayMarchingContext.bApplyShadowTransmittance);
#endif

				float3 ScatteringCoefficient = Albedo * Extinction;
				float IsotropicPhase = 1.0 / (4.0 * PI);
				RadianceSample = Inscattering * ScatteringCoefficient * IsotropicPhase * Transmittance * StepSize;
			}
#if 0
			// Stochastically decide to record scatter position
			if (any(Transmittance < RandomSample))
			{
				LocalScatterTMin = min(LocalHitT, LocalScatterTMin);
			}
#endif
			if (GetFogInscatteringMode() == FOG_INSCATTERING_MODE_REFERENCE)
			{
				float WorldRayT = LocalHitT * RayMarchingContext.LocalToWorldScale;
				float3 CameraRelative_WorldPosition = RayMarchingContext.WorldRayDirection * WorldRayT;
				float4 HeightFog = float4(0, 0, 0, 1);
				if (ShouldApplyHeightFog())
				{
					HeightFog = CalculateHeightFog(CameraRelative_WorldPosition);
				}

				float4 CombinedFog = HeightFog;
				if (ShouldApplyVolumetricFog() && FogStruct.ApplyVolumetricFog > 0)
				{
					float3 WorldPosition = CameraRelative_WorldPosition + RayMarchingContext.WorldRayOrigin;
					const uint EyeIndex = 0;
					float3 VolumeUV = ComputeVolumeUV_DEPRECATED(WorldPosition, DFHackToFloat(PrimaryView.WorldToClip));
					CombinedFog = CombineVolumetricFog(HeightFog, VolumeUV, EyeIndex, WorldRayT);
				}

				// Attenuate RadianceSample
				RadianceSample *= CombinedFog.a;

				// Add FogRadiance
				float3 DeltaTransmittance = clamp(Transmittance / PrevTransmittance, 0.0, 1.0);
				float3 DeltaFogRadiance = CombinedFog.rgb - PrevCombinedFog.rgb;
				FogRadiance += DeltaFogRadiance * DeltaTransmittance;

				PrevCombinedFog = CombinedFog;
			}
			else if (GetFogInscatteringMode() == FOG_INSCATTERING_MODE_LINEAR_APPROX)
			{
				float WorldRayT = LocalHitT * RayMarchingContext.LocalToWorldScale;
				float Weight = saturate((WorldRayT - WorldFogHitT[0]) / (WorldFogHitT[1] - WorldFogHitT[0]));
				float4 CombinedFog = lerp(FogHitRadiance[0], FogHitRadiance[1], Weight);

				// Attenuate RadianceSample
				RadianceSample *= CombinedFog.a;

				// Add FogRadiance
				float3 DeltaTransmittance = clamp(Transmittance / PrevTransmittance, 0.0, 1.0);
				float3 DeltaFogRadiance = CombinedFog.rgb - PrevCombinedFog.rgb;
				FogRadiance += DeltaFogRadiance * DeltaTransmittance;

				PrevCombinedFog = CombinedFog;
			}

			// Accumulate radiance
			Radiance += RadianceSample;
		}

#if USE_CAMERA_AVSM
		//Transmittance = AVSM_Sampler4_Eval_NoInterpolation(TransmittanceSampler, WorldHitT);
		Transmittance = AVSM_Sampler4_Eval(TransmittanceSampler, WorldHitT);
#else
		if (any(Extinction > 0.0))
		{
			// NOTE: Trimming the end point is too aggressive for exponential interpolation
			//RayMarchingContext.LocalScatterT1 = LocalHitT;
			RayMarchingContext.LocalScatterT1 = RayMarchingContext.LocalRayTMax;
			if (bFirstScatterEvent)
			{
				RayMarchingContext.LocalScatterT0 = LocalScatterTMin = LocalHitT;
				RayMarchingContext.Transmittance0 = Transmittance;
				bFirstScatterEvent = false;
			}
		}

		// Accumulate transmittance for the next evaluation
		Transmittance *= exp(-Extinction * StepSize);
#endif

		float Epsilon = 1.0e-7;
		if (all(Transmittance < Epsilon))
		{
			Transmittance = 0.0;
			break;
		}

		LocalHitT += StepSize;
	}

	if (GetFogInscatteringMode() != FOG_INSCATTERING_MODE_OFF)
	{
		// Simplifying terms to avoid vector division:
		// float3 DeltaTransmittance = clamp(Transmittance / PrevTransmittance, 0.0, 1.0);
		// float3 DeltaOpacity = 1.0 - DeltaTransmittance;
		// float3 FogAlpha = PrevTransmittance * DeltaOpacity;
		float3 FogAlpha = clamp(PrevTransmittance - Transmittance, 0.0, 1.0);

		// Fogging in-scattering contribution is not currently attenuated by Heterogeneous Volumes
		// for elements behind the subject. To prevent doubled contribution of in-scattering for 
		// regions of low density (eg: silhouette edges), blending is interpreted as alpha-masking instead.
		Radiance += FogRadiance.rgb * FogAlpha;
	}
}

void RayMarchSingleScatteringReferenceFastPath(
	inout FRayMarchingContext RayMarchingContext,
#if USE_CAMERA_AVSM
	inout FAVSM_Sampler4 TransmittanceSampler,
#endif
	uint StepCount,
	inout float3 Radiance,
	inout float3 Transmittance
)
{
	for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
	{
		float StepSize = RayMarchingContext.StepSize;
		float LocalHitT = RayMarchingContext.LocalRayTMin + RayMarchingContext.StepSize * (RayMarchingContext.Jitter + StepIndex);
		float WorldHitT = LocalHitT * RayMarchingContext.LocalToWorldScale;
		float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
		float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * WorldHitT;
#if USE_ANALYTIC_DERIVATIVES
		float3 WorldPosition_DDX = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDX * WorldHitT;
		float3 WorldPosition_DDY = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection_DDY * WorldHitT;
		WorldPosition_DDX -= WorldPosition;
		WorldPosition_DDY -= WorldPosition;
#else
		float3 WorldPosition_DDX = 0.0;
		float3 WorldPosition_DDY = 0.0;
#endif // USE_ANALYTIC_DERIVATIVES

		FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, WorldPosition_DDX, WorldPosition_DDY, RayMarchingContext.MipLevel);

		// Radiance from emission
		if (RayMarchingContext.bApplyEmissionAndTransmittance)
		{
			float3 Emission = SampleEmission(SampleContext);
			Radiance += Emission * StepSize * Transmittance;

#if DIM_USE_LUMEN_GI
			FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(DFPromote(WorldPosition), PrimaryView.WorldToClip, true);

			FTwoBandSHVector RotatedHGZonalHarmonic;
			float3 CameraVector = normalize(RayMarchingContext.WorldRayDirection);
			float PhaseG = 0.0;
			RotatedHGZonalHarmonic.V = float4(1.0f, CameraVector.y, CameraVector.z, CameraVector.x) * float4(1.0f, PhaseG, PhaseG, PhaseG);
			float3 IndirectInscattering = max(DotSH(TranslucencyGISH, RotatedHGZonalHarmonic), 0) / PI;
			IndirectInscattering *= GetIndirectInscatteringFactor();

			float3 Extinction = SampleExtinction(SampleContext);
			float3 Albedo = SampleAlbedo(SampleContext);
			float3 ScatteringCoefficient = Albedo * Extinction;
			float AmbientOcclusion = EvalAmbientOcclusion(WorldPosition);

			// Note: Phase is accounted for in spherical hamonic calculation
			Radiance += IndirectInscattering * AmbientOcclusion * ScatteringCoefficient * Transmittance * StepSize;
#endif // DIM_USE_LUMEN_GI
		}

		// Radiance from in-scattering
		float3 Extinction = SampleExtinction(SampleContext);
		if (RayMarchingContext.bApplyDirectLighting && (any(Extinction > 0)))
		{
			float3 RadianceSample = 0;

			// Evaluate in-scattering
			float3 Albedo = SampleAlbedo(SampleContext);
			if (any(Albedo > 0.0))
			{
				// Evaluate lighting cache
				float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
				float3 UVW = saturate((LocalPosition - LocalBoundsMin) / (2.0 * GetLocalBoundsExtent()));
				float3 Inscattering = SampleLightingCache(UVW, 0) * View.OneOverPreExposure;

				// Evaluate scattering
				float3 ScatteringCoefficient = Albedo * Extinction;
				float IsotropicPhase = 1.0 / (4.0 * PI);
				RadianceSample = Inscattering * ScatteringCoefficient * IsotropicPhase * Transmittance * StepSize;
			}

			Radiance += RadianceSample;
		}

		// Accumulate transmittance for the next evaluation
#if USE_CAMERA_AVSM
		Transmittance = AVSM_Sampler4_Eval(TransmittanceSampler, WorldHitT);
#else
		Transmittance *= exp(-Extinction * StepSize);
#endif

		float Epsilon = 1.0e-7;
		if (all(Transmittance < Epsilon))
		{
			Transmittance = 0.0;
			break;
		}
	}
}
