// Copyright Epic Games, Inc. All Rights Reserved.

#define SUPPORT_CONTACT_SHADOWS 0

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#include "../Common.ush"
#include "../DeferredLightingCommon.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#define SHADOW_TYPE_VOLUMETRIC_SHADOW_MAP 0
#define SHADOW_TYPE_BEER_SHADOW_MAP 1

#ifndef SHADOW_TYPE
#define SHADOW_TYPE SHADOW_TYPE_VOLUMETRIC_SHADOW_MAP
#endif // SHADOW_TYPE

// Light data
int LightType;

// Dispatch data
int3 GroupCount;
int DownsampleFactor;

// Shadow mask
Texture2D<float4> ShadowMaskTexture;

// Output
RWTexture2D<float4> RWVolumetricShadowMaskTexture;

int2 GetDownsampledResolution(int2 Resolution, int Factor)
{
	return (Resolution + Factor - 1) / Factor;
}

int2 GetScaledViewRect()
{
	return GetDownsampledResolution(View.ViewSizeAndInvSize.xy, DownsampleFactor);
}

float SampleTransmittance(
	float3 TranslatedWorldPosition,
	float3 LightTranslatedWorldPosition
)
{
#if SHADOW_TYPE == SHADOW_TYPE_VOLUMETRIC_SHADOW_MAP
	return AVSM_SampleTransmittance(TranslatedWorldPosition, LightTranslatedWorldPosition);
#else
	return BeerShadowMap_SampleTransmittance(TranslatedWorldPosition, LightTranslatedWorldPosition);
#endif
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderVolumetricShadowMaskCS(
	uint GroupId : SV_GroupID,
	uint GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	if (any(DispatchThreadId.xy >= GetScaledViewRect()))
	{
		return;
	}
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy / DownsampleFactor;

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Initialize as input
	RWVolumetricShadowMaskTexture[PixelCoord] = ShadowMaskTexture[PixelCoord];

	// Query Beer Shadow Map
	const float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(float4(PixelCoord + 0.5, DeviceZ, 1.0));
	float Transmittance = SampleTransmittance(TranslatedWorldPosition, GetDeferredLightTranslatedWorldPosition());
	RWVolumetricShadowMaskTexture[PixelCoord] *= sqrt(Transmittance);
}