// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesSparseVoxelUtils.ush"
#include "../Common.ush"

#include "HeterogeneousVolumesSparseVoxelUniformBufferUtils.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

RWBuffer<uint> RWAllocatorBuffer;

[numthreads(1, 1, 1)]
void ClearAllocator(
	uint DispatchThreadId : SV_DispatchThreadID
)
{
	RWAllocatorBuffer[0] = 0;
	RWAllocatorBuffer[1] = 1;
	RWAllocatorBuffer[2] = 1;
}

Texture3D VoxelMinTexture;
uint3 VoxelMinResolution;

// Volume data
int3 VolumeResolution;
int MipLevel;

RWStructuredBuffer<FVoxelDataPacked> RWVoxelBuffer;
RWBuffer<uint> RWNumVoxelsBuffer;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void AllocateSparseVoxels(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 VoxelIndex = DispatchThreadId;
	if (all(VoxelIndex < VoxelMinResolution))
	{
		float3 Value = VoxelMinTexture.Load(int4(VoxelIndex, 0)).xyz;
		if (dot(Value, Value) > 0.0)
		{
			// Allocate
			uint EntryIndex;
			InterlockedAdd(RWNumVoxelsBuffer[0], 1, EntryIndex);

			// Assign
			RWVoxelBuffer[EntryIndex] = EncodeVoxelData(VoxelIndex, MipLevel, VoxelMinResolution);
		}
	}
}

RWStructuredBuffer<FVoxelDataPacked> RWVoxelsToRefineBuffer;
RWBuffer<uint> RWNumVoxelsToRefineBuffer;

// TODO: Operate at lower MIP, using wave intrinsics to broadcast non-zero 
[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void AllocateSparseVoxelsToRefine(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	RWNumVoxelsToRefineBuffer[1] = 1;
	RWNumVoxelsToRefineBuffer[2] = 1;

	uint3 VoxelIndex = DispatchThreadId;
	if (all(VoxelIndex < VoxelMinResolution) && !any(VoxelIndex % 2))
	{
		// Load siblings
		uint ValidCount = 0;
		for (uint i = 0; i < 8; ++i)
		{
			uint3 Offset = uint3((i & 0x04) >> 2, (i & 0x02) >> 1, i & 0x01);
			float3 Value = VoxelMinTexture.Load(int4(VoxelIndex + Offset, 0)).xyz;
			if (dot(Value, Value) > 0.0)
			{
				ValidCount++;
			}
		}

		if (ValidCount == 8)
		{
			// Emit parent voxel for further refinement
			uint EntryIndex;
			InterlockedAdd(RWNumVoxelsToRefineBuffer[0], 1, EntryIndex);
			RWVoxelsToRefineBuffer[EntryIndex] = EncodeVoxelData(VoxelIndex / 2, MipLevel + 1, VoxelMinResolution / 2);
		}
		else
		{
			// Emit siblings as final voxels
			uint EntryIndex;
			uint SiblingIndex = 0;
			InterlockedAdd(RWNumVoxelsBuffer[0], ValidCount, EntryIndex);
			for (uint i = 0; i < 8; ++i)
			{
				uint3 Offset = uint3((i & 0x04) >> 2, (i & 0x02) >> 1, i & 0x01);
				float3 Value = VoxelMinTexture.Load(int4(VoxelIndex + Offset, 0)).xyz;
				if (dot(Value, Value) > 0.0)
				{
					RWVoxelBuffer[EntryIndex + SiblingIndex++] = EncodeVoxelData(VoxelIndex + Offset, MipLevel, VoxelMinResolution);
				}
			}
		}
	}
}

StructuredBuffer<FVoxelDataPacked> VoxelsToRefineBuffer;
Buffer<uint> NumVoxelsToRefineBuffer;
uint3 MipVolumeResolution;

[numthreads(THREADGROUP_SIZE_1D, 1, 1)]
void RefineSparseVoxels(
	uint GroupThreadId : SV_GroupThreadID,
	uint DispatchThreadId : SV_DispatchThreadID
)
{
	RWNumVoxelsToRefineBuffer[1] = 1;
	RWNumVoxelsToRefineBuffer[2] = 1;

	uint Index = DispatchThreadId;
	if (Index < NumVoxelsToRefineBuffer[0])
	{
		FVoxelData VoxelData = DecodeVoxelData(VoxelsToRefineBuffer[Index], VolumeResolution);

		uint3 ParentVoxelIndex = VoxelData.VoxelIndex / 2;
		uint3 AlignedVoxelIndex = ParentVoxelIndex * 2;

		// Load siblings
		bool bAllValid = true;
		for (uint i = 0; i < 8; ++i)
		{
			uint3 Offset = uint3((i & 0x04) >> 2, (i & 0x02) >> 1, i & 0x01);
			float3 Value = VoxelMinTexture.Load(int4(AlignedVoxelIndex + Offset, 0)).xyz;
			if (dot(Value, Value) <= 0.0)
			{
				bAllValid = false;
			}
		}

		if (bAllValid)
		{
			// Skip any odd voxels, the aligned sibiling will add the parent
			if (any(VoxelData.VoxelIndex % 2))
			{
				return;
			}

			// Emit parent voxel for further refinement
			uint EntryIndex;
			InterlockedAdd(RWNumVoxelsToRefineBuffer[0], 1, EntryIndex);
			RWVoxelsToRefineBuffer[EntryIndex] = EncodeVoxelData(ParentVoxelIndex, MipLevel + 1, MipVolumeResolution / 2);
		}
		else
		{
			// Emit voxel
			uint EntryIndex;
			InterlockedAdd(RWNumVoxelsBuffer[0], 1, EntryIndex);
			RWVoxelBuffer[EntryIndex] = VoxelsToRefineBuffer[Index];
		}
	}
}

// Using RWStructuredBuffer<float> instead of RWStructuredBuffer<float3> to overcome Vulkan alignment error:
// error: cannot instantiate RWStructuredBuffer with given packed alignment; 'VK_EXT_scalar_block_layout' not supported
#define VULKAN_ALIGNMENT_WORKAROUND 1
#if VULKAN_ALIGNMENT_WORKAROUND
	RWStructuredBuffer<float> RWPositionBuffer;
#else
	RWStructuredBuffer<float3> RWPositionBuffer;
#endif

[numthreads(64, 1, 1)]
void CreateSparseVoxelBLAS(
	uint GroupThreadId : SV_GroupThreadID,
	uint DispatchThreadId : SV_DispatchThreadID
)
{
	FBox GlobalBounds;
	GlobalBounds.Min = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	GlobalBounds.Max = GetLocalBoundsOrigin() + GetLocalBoundsExtent();

	uint3 VolumeResolution = GetVolumeResolution();

	if (DispatchThreadId < GetNumVoxels())
	{
		FVoxelDataPacked VoxelDataPacked = GetVoxelDataPacked(DispatchThreadId);
		FVoxelData VoxelData = DecodeVoxelData(VoxelDataPacked, VolumeResolution);
		FBox VoxelBounds = GetVoxelBoundingBox(VoxelData, VolumeResolution, GlobalBounds.Min, GlobalBounds.Max);

#if VULKAN_ALIGNMENT_WORKAROUND
		RWPositionBuffer[DispatchThreadId * 2 * 3 + 0] = VoxelBounds.Min.x;
		RWPositionBuffer[DispatchThreadId * 2 * 3 + 1] = VoxelBounds.Min.y;
		RWPositionBuffer[DispatchThreadId * 2 * 3 + 2] = VoxelBounds.Min.z;

		RWPositionBuffer[DispatchThreadId * 2 * 3 + 3] = VoxelBounds.Max.x;
		RWPositionBuffer[DispatchThreadId * 2 * 3 + 4] = VoxelBounds.Max.y;
		RWPositionBuffer[DispatchThreadId * 2 * 3 + 5] = VoxelBounds.Max.z;
#else
		RWPositionBuffer[DispatchThreadId * 2] = VoxelBounds.Min;
		RWPositionBuffer[DispatchThreadId * 2 + 1] = VoxelBounds.Max;
#endif
	}
}