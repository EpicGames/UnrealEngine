// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesStochasticFiltering.ush"

float4x4 GetLocalToWorld()
{
	return SparseVoxelUniformBuffer.LocalToWorld;
}

float4x4 GetWorldToLocal()
{
	return SparseVoxelUniformBuffer.WorldToLocal;
}

float3 GetLocalBoundsOrigin()
{
	return SparseVoxelUniformBuffer.LocalBoundsOrigin;
}

float3 GetLocalBoundsExtent()
{
	return SparseVoxelUniformBuffer.LocalBoundsExtent;
}

float GetMaxTraceDistance()
{
	return SparseVoxelUniformBuffer.MaxTraceDistance;
}

float GetMaxShadowTraceDistance()
{
	return SparseVoxelUniformBuffer.MaxShadowTraceDistance;
}

float GetStepSize()
{
	return SparseVoxelUniformBuffer.StepSize;
}

float GetStepFactor()
{
	return SparseVoxelUniformBuffer.StepFactor;
}

float GetShadowStepSize()
{
	return SparseVoxelUniformBuffer.ShadowStepSize;
}

float GetShadowStepFactor()
{
	return SparseVoxelUniformBuffer.ShadowStepFactor;
}

int GetStochasticFilteringMode()
{
	return STOCHASTIC_FILTERING_TRILINEAR;
}

int ShouldApplyHeightFog()
{
	return SparseVoxelUniformBuffer.bApplyHeightFog;
}

int ShouldApplyVolumetricFog()
{
	return SparseVoxelUniformBuffer.bApplyVolumetricFog;
}

float GetIndirectInscatteringFactor()
{
	return SparseVoxelUniformBuffer.IndirectInscatteringFactor;
}

uint3 GetVolumeResolution()
{
	return SparseVoxelUniformBuffer.VolumeResolution;
}

uint GetNumVoxels()
{
	return SparseVoxelUniformBuffer.NumVoxelsBuffer[0];
}

FVoxelDataPacked GetVoxelDataPacked(uint VoxelIndex)
{
	return SparseVoxelUniformBuffer.VoxelBuffer[VoxelIndex];
}

struct FVolumeSampleContext
{
	float3 UVW;
	float MipLevel;
};

FVolumeSampleContext CreateVolumeSampleContext(
	float3 LocalPosition,
	float3 WorldPosition,
	float3 WorldPosition_DDX,
	float3 WorldPosition_DDY,
	float MipLevel
)
{
	float3 LocalBoundsMin = GetLocalBoundsOrigin() - GetLocalBoundsExtent();
	float3 LocalBoundsMax = GetLocalBoundsOrigin() + GetLocalBoundsExtent();

	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.UVW = (LocalPosition - LocalBoundsMin) / (LocalBoundsMax - LocalBoundsMin);
	VolumeSampleContext.MipLevel = MipLevel;

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0.0;
	float3 WorldPosition_DDY = 0.0;

	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int SamplingMethod)
{
	// TODO: Use FilterWidth determine MipLevel
	//float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	//float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (SamplingMethod == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (SamplingMethod == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (SamplingMethod == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	WorldPosition = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;
	return CreateVolumeSampleContext(LocalPosition, WorldPosition, MipLevel);
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	float3 Extinction = Texture3DSampleLevel(
		SparseVoxelUniformBuffer.ExtinctionTexture,
		SparseVoxelUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).rgb;

	return Extinction;
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	float3 Emission = Texture3DSampleLevel(
		SparseVoxelUniformBuffer.EmissionTexture,
		SparseVoxelUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).xyz;

	return Emission;
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = Texture3DSampleLevel(
		SparseVoxelUniformBuffer.AlbedoTexture,
		SparseVoxelUniformBuffer.TextureSampler,
		Context.UVW,
		Context.MipLevel).xyz;

	return Albedo;
}