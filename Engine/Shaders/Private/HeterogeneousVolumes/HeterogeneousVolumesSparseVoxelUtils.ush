// Copyright Epic Games, Inc. All Rights Reserved.

#define INVALID_VOXEL 0xFFFFFFFF;

struct FVoxelData
{
	uint3 VoxelIndex;
	uint MipLevel;
};

struct FVoxelDataPacked
{
	uint LinearIndex;
	uint MipLevel;
};

FVoxelDataPacked EncodeVoxelData(uint3 VoxelIndex, uint MipLevel, uint3 VolumeResolutionAtMipLevel)
{
	FVoxelDataPacked VoxelDataPacked = (FVoxelDataPacked)0;

	// TODO: Morton Order
	VoxelDataPacked.LinearIndex =
		VoxelIndex.z * VolumeResolutionAtMipLevel.x * VolumeResolutionAtMipLevel.y +
		VoxelIndex.y * VolumeResolutionAtMipLevel.x +
		VoxelIndex.x;
	VoxelDataPacked.MipLevel = MipLevel;

	return VoxelDataPacked;
}

FVoxelData DecodeVoxelData(FVoxelDataPacked VoxelDataPacked, uint3 VolumeResolutionAtMip0)
{
	FVoxelData VoxelData;
	VoxelData.MipLevel = VoxelDataPacked.MipLevel;

	// TODO: Morton Order
	uint3 VolumeRes = VolumeResolutionAtMip0 >> VoxelData.MipLevel;
	VoxelData.VoxelIndex.x = (VoxelDataPacked.LinearIndex % (VolumeRes.x * VolumeRes.y)) % VolumeRes.x;
	VoxelData.VoxelIndex.y = (VoxelDataPacked.LinearIndex % (VolumeRes.x * VolumeRes.y)) / VolumeRes.x;
	VoxelData.VoxelIndex.z = VoxelDataPacked.LinearIndex / (VolumeRes.x * VolumeRes.y);

	return VoxelData;
}

FVoxelData CreateEmptyVoxelData()
{
	FVoxelData VoxelData;
	VoxelData.VoxelIndex = 0;
	VoxelData.MipLevel = INVALID_VOXEL;
	return VoxelData;
}

bool IsVoxelDataValid(FVoxelData VoxelData)
{
	return VoxelData.MipLevel != INVALID_VOXEL;
}

bool IsVoxelDataEqual(FVoxelData LHS, FVoxelData RHS)
{
	return LHS.VoxelIndex.x == RHS.VoxelIndex.x &&
		LHS.VoxelIndex.y == RHS.VoxelIndex.y &&
		LHS.VoxelIndex.z == RHS.VoxelIndex.z &&
		LHS.MipLevel == RHS.MipLevel;
}

struct FBox
{
	float3 Min;
	float3 Max;
};

FBox GetVoxelBoundingBox(FVoxelData VoxelData, int3 VolumeResolution, float3 ObjectBoundsMin, float3 ObjectBoundsMax)
{
	float3 MipResolution = float3(
		VolumeResolution.x >> VoxelData.MipLevel,
		VolumeResolution.y >> VoxelData.MipLevel,
		VolumeResolution.z >> VoxelData.MipLevel
	);

	float3 UVW0 = VoxelData.VoxelIndex / MipResolution;
	float3 UVW1 = (VoxelData.VoxelIndex + 1) / MipResolution;
	float3 ObjectBoundsExtent = ObjectBoundsMax - ObjectBoundsMin;

	FBox Box;
	const float Epsilon = 0.0001;
	Box.Min = ObjectBoundsMin + ObjectBoundsExtent * UVW0 - Epsilon;
	Box.Max = ObjectBoundsMin + ObjectBoundsExtent * UVW1 + Epsilon;
	return Box;
}
