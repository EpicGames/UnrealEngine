// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

int3 WeightedReservoir(int i, int j, float Prob, int3 Index, inout float Rand)
{
	if (Rand < Prob)
	{
		Index[i] = j;
		Rand /= Prob;
	}
	else
	{
		Rand = (Rand - Prob) / (1.0 - Prob);
	}

	return Index;
}

#define STOCHASTIC_FILTERING_DISABLED 0
#define STOCHASTIC_FILTERING_CONSTANT 1
#define STOCHASTIC_FILTERING_TRILINEAR 2
#define STOCHASTIC_FILTERING_TRICUBIC 3

int3 StochasticFilteringConstant(float3 Position, inout float Rand)
{
	return Position;
}

int3 StochasticFilteringTrilinear(float3 Position, inout float Rand)
{
	int3 PositionInt = floor(Position);
	float3 Delta = Position - PositionInt;

	int3 Index = 0;
	for (int i = 0; i < 3; ++i)
	{
		float Prob = Delta[i];
		Index = WeightedReservoir(i, 1, Prob, Index, Rand);
	}

	return PositionInt + Index;
}

int3 StochasticFilteringTricubic(float3 Position, inout float Rand)
{
	int3 PositionInt = floor(Position);
	float3 Delta = Position - PositionInt;

	int3 Index = 0;
	for (int i = 0; i < 3; ++i)
	{
		float t = Delta[i];
		float t2 = t * t;

		float4 Weights = float4(
			(1.0 / 6.0) * (-t*t2 + 3*t2 - 3*t + 1),
			(1.0 / 6.0) * (3*t*t2 - 6*t2 + 4),
			(1.0 / 6.0) * (-3*t*t2 + 3*t2 + 3*t + 1),
			(1.0 / 6.0) * t*t2
		);

		float Sum = Weights[0];
		for (int j = 0; j < 3; ++j)
		{
			Sum += Weights[j + 1];
			float Prob = Weights[j + 1] / Sum;
			Index = WeightedReservoir(i, j + 1, Prob, Index, Rand);
		}
	}

	return PositionInt - 1 + Index;
}
