// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../VelocityCommon.ush"

// Modified LineBoxIntersect() to operate at consistent RayDirection scale, within some [TMin, TMax] span
float2 IntersectAABB(float3 RayOrigin, float3 RayDirection, float RayTMin, float RayTMax, float3 BoxMin, float3 BoxMax)
{
	float3 InvRayDir = 1.0 / RayDirection;
	//find the ray intersection with each of the 3 planes defined by the minimum extrema.
	float3 FirstPlaneIntersections = (BoxMin - RayOrigin) * InvRayDir;
	//find the ray intersection with each of the 3 planes defined by the maximum extrema.
	float3 SecondPlaneIntersections = (BoxMax - RayOrigin) * InvRayDir;
	//get the closest of these intersections along the ray
	float3 ClosestPlaneIntersections = min(FirstPlaneIntersections, SecondPlaneIntersections);
	//get the furthest of these intersections along the ray
	float3 FurthestPlaneIntersections = max(FirstPlaneIntersections, SecondPlaneIntersections);

	float2 BoxIntersections;
	//find the furthest near intersection
	BoxIntersections.x = max(ClosestPlaneIntersections.x, max(ClosestPlaneIntersections.y, ClosestPlaneIntersections.z));
	//find the closest far intersection
	BoxIntersections.y = min(FurthestPlaneIntersections.x, min(FurthestPlaneIntersections.y, FurthestPlaneIntersections.z));
	//clamp the intersections to be between RayTMin and RayTMax
	return float2(max(BoxIntersections.x, RayTMin), min(BoxIntersections.y, RayTMax));
}

float3 Calculate3DVelocityAtWorldPos(float3 WorldRayOrigin, float3 WorldRayDirection, float WorldHitT)
{
	float3 WorldPos = WorldRayOrigin + WorldRayDirection * WorldHitT;
	float3 TranslatedWorldPos = DFHackToFloat(DFFastAdd(WorldPos, PrimaryView.PreViewTranslation));
	float4 ClipPos = mul(float4(TranslatedWorldPos, 1), View.TranslatedWorldToClip);
	ClipPos /= ClipPos.w;

	float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
	float3 PrevTranslatedWorldPos = mul(float4(GetScreenPositionForProjectionType(ClipPos.xy, WorldHitT), WorldHitT, 1), View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;
	float4 PrevClipPos = mul(float4(PrevTranslatedWorldPos, 1), View.TranslatedWorldToClip);
	PrevClipPos /= PrevClipPos.w;

	return Calculate3DVelocity(ClipPos, PrevClipPos);
}