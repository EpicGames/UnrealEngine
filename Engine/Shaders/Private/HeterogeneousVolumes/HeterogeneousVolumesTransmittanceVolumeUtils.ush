// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef DIM_USE_TRANSMITTANCE_VOLUME
#define DIM_USE_TRANSMITTANCE_VOLUME 0
#endif // DIM_USE_TRANSMITTANCE_VOLUME

#define LIGHTING_CACHE_TRANSMITTANCE 0
#define LIGHTING_CACHE_INSCATTERING 1

uint3 LightingCacheResolution;
float LightingCacheVoxelBias;
Texture3D LightingCacheTexture;

uint3 GetLightingCacheResolution()
{
	return LightingCacheResolution;
}

float GetLightingCacheVoxelBias()
{
	return LightingCacheVoxelBias;
}

float3 SampleLightingCache(float3 UVW, float MipLevel)
{
	float3 Transmittance = Texture3DSampleLevel(
		LightingCacheTexture,
		View.SharedTrilinearClampedSampler,
		UVW,
		MipLevel).xyz;

	return Transmittance;
}
