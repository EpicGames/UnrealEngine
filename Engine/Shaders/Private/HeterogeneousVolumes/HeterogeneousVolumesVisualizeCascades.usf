// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"

int DownsampleFactor;

Texture2D SceneDepthTexture;
RWTexture2D<float4> RWVisualizationTexture;

struct FAdaptiveVolumetricShadowMap
{
	float4x4 TranslatedWorldToShadow[6];
	float3 TranslatedWorldOrigin[6];
	float4 TranslatedWorldPlane[6];
	float4 SplitDepths[6];
	float DownsampleFactor;

	int2 Resolution;
	int NumShadowMatrices;
	int MaxSampleCount;
	bool bIsEmpty;
	bool bIsDirectionalLight;

	StructuredBuffer<uint4> IndirectionBuffer;
	StructuredBuffer<uint> SampleBuffer;
	Texture2D<float4> RadianceTexture;
	SamplerState TextureSampler;
};

FAdaptiveVolumetricShadowMap GetAdaptiveVolumetricShadowMap()
{
	FAdaptiveVolumetricShadowMap ShadowMap;
	for (int i = 0; i < 6; ++i)
	{
		ShadowMap.TranslatedWorldToShadow[i] = AVSM.TranslatedWorldToShadow[i];
		ShadowMap.TranslatedWorldOrigin[i] = AVSM.TranslatedWorldOrigin[i].xyz;
		ShadowMap.TranslatedWorldPlane[i] = AVSM.TranslatedWorldPlane[i];
		ShadowMap.SplitDepths[i] = AVSM.SplitDepths[i];
	}

	ShadowMap.Resolution = AVSM.Resolution;
	ShadowMap.DownsampleFactor = AVSM.DownsampleFactor;
	ShadowMap.NumShadowMatrices = AVSM.NumShadowMatrices;
	ShadowMap.MaxSampleCount = AVSM.MaxSampleCount;
	ShadowMap.bIsEmpty = AVSM.bIsEmpty;
	ShadowMap.bIsDirectionalLight = AVSM.bIsDirectionalLight;

	ShadowMap.IndirectionBuffer = AVSM.IndirectionBuffer;
	ShadowMap.SampleBuffer = AVSM.SampleBuffer;

	ShadowMap.RadianceTexture = AVSM.RadianceTexture;
	ShadowMap.TextureSampler = AVSM.TextureSampler;
	return ShadowMap;
}

int2 GetDownsampledResolution(int2 Resolution, int Factor)
{
	return (Resolution + Factor - 1) / Factor;
}

int2 GetScaledViewRect()
{
	return GetDownsampledResolution(View.ViewSizeAndInvSize.xy, DownsampleFactor);
}

float3 FaceToColor(int Face)
{
	if (Face == 3)
	{
		return float3(0, 1, 0);
	}
	else if (Face == 2)
	{
		return float3(0, 1, 1);
	}
	else if (Face == 1)
	{
		return float3(0, 0, 1);
	}

	//else
	return float3(1, 0, 1);
}

float GetMaxTraceDistance()
{
	//return 30000.0f;
	return 100000.0f;
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void VisualizeCascadesCS(
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	// Create screen ray
	if (any(DispatchThreadId.xy >= GetScaledViewRect()))
	{
		return;
	}

	uint2 ViewPixelCoord = DispatchThreadId.xy * DownsampleFactor + View.ViewRectMin.xy;
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy / DownsampleFactor;

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(ViewPixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float Depth = min(ConvertFromDeviceZ(DeviceZ), GetMaxTraceDistance());

	FAdaptiveVolumetricShadowMap ShadowMap = GetAdaptiveVolumetricShadowMap();
	int Face = ShadowMap.NumShadowMatrices - 1;
#if 1
	for (; Face > 0; --Face)
	{
		if (Depth < ShadowMap.SplitDepths[Face].y)
		{
			break;
		}
	}
#endif
	RWVisualizationTexture[PixelCoord] = float4(FaceToColor(Face), 1);
}