// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesVoxelGridTypes.ush"

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../ComputeShaderUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesVoxelGridTraversal.ush"

#ifndef THREADGROUP_SIZE_1D
#define THREADGROUP_SIZE_1D 1
#endif // THREADGROUP_SIZE_1D

#ifndef THREADGROUP_SIZE_2D
#define THREADGROUP_SIZE_2D 1
#endif // THREADGROUP_SIZE_2D

#ifndef THREADGROUP_SIZE_3D
#define THREADGROUP_SIZE_3D 1
#endif // THREADGROUP_SIZE_3D

#ifndef DIM_ENABLE_INDIRECTION_GRID
#define DIM_ENABLE_INDIRECTION_GRID 0
#endif // DIM_ENABLE_INDIRECTION_GRID

int3 TopLevelGridResolution;
float3 TopLevelGridWorldBoundsMin;
float3 TopLevelGridWorldBoundsMax;

StructuredBuffer<FTopLevelGridBitmaskData> TopLevelGridBitmaskBuffer;
StructuredBuffer<FTopLevelGridData> TopLevelGridBuffer;
StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer;
StructuredBuffer<FScalarGridData> ExtinctionGridBuffer;
StructuredBuffer<FVectorGridData> EmissionGridBuffer;
StructuredBuffer<FVectorGridData> ScatteringGridBuffer;

float3 PrimitiveWorldBoundsMin;
float3 PrimitiveWorldBoundsMax;

float ShadingRateInFrustum;
float ShadingRateOutOfFrustum;
float MinVoxelSizeInFrustum;
float MinVoxelSizeOutOfFrustum;
float DownsampleFactor;
int bUseProjectedPixelSize;

RWStructuredBuffer<FTopLevelGridData> RWTopLevelGridBuffer;

void CalcVoxelBounds(float3 VoxelIndex, float3 GridResolution, float3 WorldBoundsMin, float3 WorldBoundsMax,
	inout float3 VoxelBoundsMin, inout float3 VoxelBoundsMax)
{
	float3 WorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	VoxelBoundsMin = WorldBoundsMin + WorldBoundsExtent * (VoxelIndex / GridResolution);
	VoxelBoundsMax = WorldBoundsMin + WorldBoundsExtent * ((VoxelIndex + 1) / GridResolution);
}

void CalcTopLevelVoxelBounds(float3 VoxelIndex, inout float3 VoxelBoundsMin, inout float3 VoxelBoundsMax)
{
	CalcVoxelBounds(VoxelIndex, TopLevelGridResolution, TopLevelGridWorldBoundsMin, TopLevelGridWorldBoundsMax,
		VoxelBoundsMin, VoxelBoundsMax);
}

float3 GetVoxelCenter(float3 VoxelIndex, inout float3 VoxelBoundsMin, inout float3 VoxelBoundsMax)
{
	float3 TopLevelGridWorldBoundsExtent = TopLevelGridWorldBoundsMax - TopLevelGridWorldBoundsMin;
	return TopLevelGridWorldBoundsMin + TopLevelGridWorldBoundsExtent * ((VoxelIndex + 0.5) / TopLevelGridResolution);
}

bool PrimitiveIntersectsVoxel(float3 VoxelBoundsMin, float3 VoxelBoundsMax)
{
	if (any(PrimitiveWorldBoundsMin > VoxelBoundsMax) || any(VoxelBoundsMin > PrimitiveWorldBoundsMax))
	{
		return false;
	}

	return true;
}

// Copied from LiveShadingPipeline.usf
float4 CreatePlane(
	float3 P_000,
	float3 P_100,
	float3 P_010
)
{
	float3 X = P_100 - P_000;
	float3 Y = P_010 - P_000;

	float3 N = normalize(cross(X, Y));
	float d = -dot(N, P_000);
	return float4(N, d);
}

float CalcSignedDistance(float3 P, float4 Plane)
{
	return dot(P, Plane.xyz) + Plane.w;
}

bool BoxIntersectsFrustum(float3 BoxOrigin, float3 BoxExtent, float4 Planes[5])
{
	for (int i = 0; i < 5; ++i)
	{
		float Distance = CalcSignedDistance(BoxOrigin, Planes[i]);
		float ProjectedExtent = dot(BoxExtent, abs(Planes[i].xyz));

		if (Distance > ProjectedExtent)
		{
			return false;
		}
	}

	return true;
}

bool BoxIntersectsViewFrustum(float3 BoxOrigin, float3 BoxExtent)
{
	float Epsilon = 1.0e-6;
	float NearDepth = 1.0;
	float FarDepth = Epsilon;

	FDFVector3 BoxWorldOrigin = DFPromote(BoxOrigin); // LWC_TODO
	float3 BoxTranslatedWorldOrigin = DFFastToTranslatedWorld(BoxWorldOrigin, PrimaryView.PreViewTranslation);

	// Project frustum vertices into world-space
	float4 Viewport = float4(0.0, 0.0, View.ViewSizeAndInvSize.xy);
	float3 W_000 = SvPositionToTranslatedWorld(float4(Viewport.xw, NearDepth, 1));
	float3 W_100 = SvPositionToTranslatedWorld(float4(Viewport.zw, NearDepth, 1));
	float3 W_010 = SvPositionToTranslatedWorld(float4(Viewport.xy, NearDepth, 1));
	float3 W_110 = SvPositionToTranslatedWorld(float4(Viewport.zy, NearDepth, 1));
	float3 W_001 = SvPositionToTranslatedWorld(float4(Viewport.xw, FarDepth, 1));
	float3 W_101 = SvPositionToTranslatedWorld(float4(Viewport.zw, FarDepth, 1));
	float3 W_011 = SvPositionToTranslatedWorld(float4(Viewport.xy, FarDepth, 1));
	float3 W_111 = SvPositionToTranslatedWorld(float4(Viewport.zy, FarDepth, 1));

	float4 Planes[5] = {
		// Front
		CreatePlane(W_000, W_010, W_100),
		// Back
		//CreatePlane(W_101, W_001, W_111),
		// +X
		CreatePlane(W_100, W_110, W_101),
		// +Y
		CreatePlane(W_010, W_011, W_110),
		// -X
		CreatePlane(W_000, W_001, W_010),
		// -Y
		CreatePlane(W_000, W_100, W_001)
	};

	return BoxIntersectsFrustum(BoxTranslatedWorldOrigin, BoxExtent, Planes);
}

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void TopLevelGridCalculateVoxelSize(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	float3 VoxelBoundsMin;
	float3 VoxelBoundsMax;
	CalcTopLevelVoxelBounds(VoxelIndex, VoxelBoundsMin, VoxelBoundsMax);

	float3 WorldVoxelCenter = (VoxelBoundsMin + VoxelBoundsMax) * 0.5;
	float3 WorldVoxelExtent = (VoxelBoundsMax - VoxelBoundsMin) * 0.5;

	if (PrimitiveIntersectsVoxel(VoxelBoundsMin, VoxelBoundsMax))
	{
		float VoxelSize = MinVoxelSizeOutOfFrustum;
		float ShadingRate = ShadingRateOutOfFrustum;
		if (BoxIntersectsViewFrustum(WorldVoxelCenter, WorldVoxelExtent))
		{
			VoxelSize = MinVoxelSizeInFrustum;
			ShadingRate = ShadingRateInFrustum;
		}

		if (bUseProjectedPixelSize)
		{
			float2 TanHalfFOV = GetTanHalfFieldOfView();
			float HalfWidth = View.ViewSizeAndInvSize.x * 0.5 / DownsampleFactor;
			float PixelWidth = TanHalfFOV.x / HalfWidth;

			float3 WorldCameraOrigin = DFFastSubtractDemote(View.TranslatedWorldCameraOrigin, PrimaryView.PreViewTranslation);
			float Distance = max(GetDistanceToCameraFromViewVector(WorldVoxelCenter - WorldCameraOrigin) - length(WorldVoxelExtent), 0.0);
			float VoxelWidth = Distance * PixelWidth * ShadingRate;
			VoxelSize = max(VoxelWidth, VoxelSize);
		}

		// Store minimum voxel rate temporarily as the bottom-level index
		uint LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
		if (IsBottomLevelAllocated(RWTopLevelGridBuffer[LinearIndex]))
		{
			float PrevVoxelSize = GetVoxelSize(RWTopLevelGridBuffer[LinearIndex]);
			VoxelSize = min(VoxelSize, PrevVoxelSize);
		}

		FTopLevelGridData TopLevelGridData;
		{
			SetVoxelSize(TopLevelGridData, VoxelSize);
		}
		RWTopLevelGridBuffer[LinearIndex] = TopLevelGridData;
	}
}

int3 NextPowerOfTwo(int3 Value)
{
	Value -= 1;
	Value |= Value >> 1;
	Value |= Value >> 2;
	Value |= Value >> 4;
	Value |= Value >> 8;
	Value |= Value >> 16;
	Value += 1;

	return Value;
}

int MaxVoxelResolution;
int bSampleAtVertices;

// Indirection Grid
int IndirectionGridBufferSize;
RWBuffer<uint> RWIndirectionGridAllocatorBuffer;
RWStructuredBuffer<FTopLevelGridData> RWIndirectionGridBuffer;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void AllocateBottomLevelGrid(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	uint LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
	if (IsBottomLevelAllocated(RWTopLevelGridBuffer[LinearIndex]))
	{
		float VoxelSize = GetVoxelSize(RWTopLevelGridBuffer[LinearIndex]);

		float3 VoxelBoundsMin;
		float3 VoxelBoundsMax;
		CalcTopLevelVoxelBounds(VoxelIndex, VoxelBoundsMin, VoxelBoundsMax);

		// Calculate bottom-level voxel resolution and round to nearest power-of-two
		float3 VoxelBoundsExtent = VoxelBoundsMax - VoxelBoundsMin;
		int3 VoxelResolution = NextPowerOfTwo(ceil(VoxelBoundsExtent / VoxelSize));
		if (bSampleAtVertices)
		{
			VoxelResolution = max(VoxelResolution, 2);
		}

		// DEBUG: OVERRIDE RESOLUTION..
		// VoxelResolution = MaxVoxelResolution;
		//VoxelResolution = max(VoxelResolution.x, max(VoxelResolution.y, VoxelResolution.z));

#if DIM_ENABLE_INDIRECTION_GRID
		// Allocate indirection voxels
		int3 BottomLevelVoxelResolution = MaxVoxelResolution;
		int3 IndirectionGridResolution = ceil(VoxelResolution / BottomLevelVoxelResolution);
		IndirectionGridResolution = clamp(IndirectionGridResolution, 1, 4);
		// Force regular sized dimensions so the permutation count is tractable for the allocator
		IndirectionGridResolution = max(IndirectionGridResolution.x, max(IndirectionGridResolution.y, IndirectionGridResolution.z));
		int IndirectionVoxelCount = IndirectionGridResolution.x * IndirectionGridResolution.y * IndirectionGridResolution.z;

		uint IndirectionGridIndex = EMPTY_VOXEL_INDEX;
		InterlockedAdd(RWIndirectionGridAllocatorBuffer[0], IndirectionVoxelCount, IndirectionGridIndex);

		// If we over-allocate, eclare the allocator to be full and the voxel contents to be empty
		if (IndirectionGridIndex + IndirectionVoxelCount > IndirectionGridBufferSize)
		{
			uint Dummy;
			InterlockedExchange(RWIndirectionGridAllocatorBuffer[0], IndirectionGridBufferSize, Dummy);

			IndirectionGridIndex = EMPTY_VOXEL_INDEX;
			IndirectionGridResolution = 0;
		}

		// Write the voxel contents
		FTopLevelGridData TopLevelGridData = (FTopLevelGridData)0;
		{
			SetBottomLevelIndex(TopLevelGridData, IndirectionGridIndex);
			SetBottomLevelVoxelResolution(TopLevelGridData, IndirectionGridResolution);
		}
		RWTopLevelGridBuffer[LinearIndex] = TopLevelGridData;

#else // DIM_ENABLE_INDIRECTION_GRID
		uint BottomLevelIndex = EMPTY_VOXEL_INDEX;

		VoxelResolution = clamp(VoxelResolution, 1, MaxVoxelResolution);
		// Force regular sized dimensions so the permutation count is tractable for the allocator
		VoxelResolution = max(VoxelResolution.x, max(VoxelResolution.y, VoxelResolution.z));

		// Write the voxel contents
		FTopLevelGridData TopLevelGridData = RWTopLevelGridBuffer[LinearIndex];
		{
			SetBottomLevelIndex(TopLevelGridData, BottomLevelIndex);
			SetBottomLevelVoxelResolution(TopLevelGridData, VoxelResolution);
		}
		RWTopLevelGridBuffer[LinearIndex] = TopLevelGridData;
#endif // DIM_ENABLE_INDIRECTION_GRID
	}
}

struct FRasterTileData
{
	uint TopLevelGridLinearIndex;
	uint BottomLevelGridLinearOffset;
};

int RasterTileVoxelResolution;
int MaxNumRasterTiles;

RWBuffer<uint> RWRasterTileAllocatorBuffer;
RWStructuredBuffer<FRasterTileData> RWRasterTileBuffer;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void GenerateRasterTiles(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	int LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
	if (!IsBottomLevelEmpty(TopLevelGridBuffer[LinearIndex]))
	{
		int3 BottomLevelResolution = GetBottomLevelVoxelResolution(TopLevelGridBuffer[LinearIndex]);

#if DIM_ENABLE_INDIRECTION_GRID
		int3 RasterTileCount3 = BottomLevelResolution;
#else // DIM_ENABLE_INDIRECTION_GRID
		// NOTE: Always 1, due to limitations of bottom-level grid resolution
		//int3 RasterTileCount3 = ceil(float3(BottomLevelResolution) / RasterTileVoxelResolution);
		int3 RasterTileCount3 = 1;
#endif // DIM_ENABLE_INDIRECTION_GRID

		int RasterTileCount = RasterTileCount3.x * RasterTileCount3.y * RasterTileCount3.z;

		if (RasterTileCount > 0)
		{
			uint RasterTileOffset;
			InterlockedAdd(RWRasterTileAllocatorBuffer[0], RasterTileCount, RasterTileOffset);

			// Prevent over allocation
			if (RasterTileOffset + RasterTileCount <= MaxNumRasterTiles)
			{
				for (uint Index = 0; Index < RasterTileCount; ++Index)
				{
					uint RasterTileIndex = RasterTileOffset + Index;
					RWRasterTileBuffer[RasterTileIndex].TopLevelGridLinearIndex = LinearIndex;
					RWRasterTileBuffer[RasterTileIndex].BottomLevelGridLinearOffset = Index;
				}
			}
			else
			{
				// Declare the buffer to be filled
				uint Dummy;
				InterlockedExchange(RWRasterTileAllocatorBuffer[0], MaxNumRasterTiles, Dummy);
			}
		}
	}
}

int3 MaxDispatchThreadGroupsPerDimension;
Buffer<uint> RasterTileAllocatorBuffer;
RWBuffer<uint> RWRasterizeBottomLevelGridIndirectArgsBuffer;

uint3 GetGroupCountWrapped(uint TargetGroupCount)
{
	const uint WrappedGroupStride = 128;

	uint3 GroupCount = uint3(TargetGroupCount, 1, 1);

	if (GroupCount.x > MaxDispatchThreadGroupsPerDimension.x)
	{
		GroupCount.y = (GroupCount.x + WrappedGroupStride - 1) / WrappedGroupStride;
		GroupCount.x = WrappedGroupStride;
	}
	if (GroupCount.y > MaxDispatchThreadGroupsPerDimension.y)
	{
		GroupCount.z = (GroupCount.y + WrappedGroupStride - 1) / WrappedGroupStride;
		GroupCount.y = WrappedGroupStride;
	}

	return GroupCount;
}

[numthreads(1, 1, 1)]
void SetRasterizeBottomLevelGridIndirectArgs()
{
	uint3 GroupCount = GetGroupCountWrapped(RasterTileAllocatorBuffer[0]);
	RWRasterizeBottomLevelGridIndirectArgsBuffer[0] = GroupCount.x;
	RWRasterizeBottomLevelGridIndirectArgsBuffer[1] = GroupCount.y;
	RWRasterizeBottomLevelGridIndirectArgsBuffer[2] = GroupCount.z;
}

RWStructuredBuffer<FTopLevelGridBitmaskData> RWTopLevelGridBitmaskBuffer;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void TopLevelCreateBitmaskCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	int3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	int LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
	if (!IsBottomLevelEmpty(TopLevelGridBuffer[LinearIndex]))
	{
		FTopLevelGridBitmaskData Bitmask = (FTopLevelGridBitmaskData) 0;

		uint BitmaskIndex = MortonEncode3(GroupThreadId);
		if (BitmaskIndex < 32)
		{
			Bitmask.PackedData[0] = 1u << BitmaskIndex;
		}
		else
		{
			Bitmask.PackedData[1] = 1u << (BitmaskIndex - 32u);
		}

		uint GroupIdLinearIndex = MortonEncode3(GroupId);
		InterlockedOr(RWTopLevelGridBitmaskBuffer[GroupIdLinearIndex].PackedData[0], Bitmask.PackedData[0]);
		InterlockedOr(RWTopLevelGridBitmaskBuffer[GroupIdLinearIndex].PackedData[1], Bitmask.PackedData[1]);
	}
}

RWStructuredBuffer<FScalarGridData> RWMajorantVoxelGridBuffer;

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void BuildMajorantVoxelGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	int3 VoxelIndex = DispatchThreadId;
	if (any(VoxelIndex >= TopLevelGridResolution))
	{
		return;
	}

	FMajorantData MajorantData = CreateMajorantData();

	int LinearIndex = GetLinearIndex(VoxelIndex, TopLevelGridResolution);
	FTopLevelGridData TopLevelGridData = TopLevelGridBuffer[LinearIndex];
	if (IsBottomLevelAllocated(TopLevelGridData))
	{
		int3 VoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);
		int BottomLevelVoxelCount = VoxelResolution.x * VoxelResolution.y * VoxelResolution.z;
#if DIM_ENABLE_INDIRECTION_GRID
		uint IndirectionGridIndex = GetBottomLevelIndex(TopLevelGridData);
		for (int Index = 0; Index < BottomLevelVoxelCount; ++Index)
		{
			FTopLevelGridData IndirectionData = IndirectionGridBuffer[IndirectionGridIndex + Index];
			if (IsBottomLevelAllocated(IndirectionData))
			{
				uint IndirectionIndex = GetBottomLevelIndex(IndirectionData);
				int3 IndirectionResolution = GetBottomLevelVoxelResolution(IndirectionData);
				int IndirectionVoxelCount = IndirectionResolution.x * IndirectionResolution.y * IndirectionResolution.z;
				for (uint VoxelIndex = 0; VoxelIndex < IndirectionVoxelCount; ++VoxelIndex)
				{
					uint BottomLevelIndex = IndirectionIndex + VoxelIndex;
					float3 Extinction = GetExtinction(ExtinctionGridBuffer[BottomLevelIndex]);
					UpdateMajorantData(MajorantData, Extinction);
				}
			}
		}
#else // DIM_ENABLE_INDIRECTION_GRID
		for (int VoxelIndex = 0; VoxelIndex < BottomLevelVoxelCount; ++VoxelIndex)
		{
			int BottomLevelIndex = GetBottomLevelIndex(TopLevelGridData) + VoxelIndex;
			float3 Extinction = GetExtinction(ExtinctionGridBuffer[BottomLevelIndex]);
			UpdateMajorantData(MajorantData, Extinction);
		}
#endif // DIM_ENABLE_INDIRECTION_GRID
	}

	SetMajorantData(RWMajorantVoxelGridBuffer[LinearIndex], MajorantData);
}

StructuredBuffer<FScalarGridData> InputBuffer;
RWStructuredBuffer<FScalarGridData> RWOutputBuffer;

int3 InputDimensions;

groupshared float GSMajorant[THREADGROUP_SIZE_1D];

[numthreads(THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D, THREADGROUP_SIZE_3D)]
void DownsampleMajorantVoxelGridCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint LinearThreadIndex = MortonEncode3(GroupThreadId);
	uint LinearDispatchIndex = GetLinearIndex(DispatchThreadId, InputDimensions);
	GSMajorant[LinearThreadIndex] = 0;
	if (all(DispatchThreadId < InputDimensions))
	{
		FMajorantData MajorantData = GetMajorantData(InputBuffer[LinearDispatchIndex]);
		GSMajorant[LinearThreadIndex] = MajorantData.Majorant;
	}

	// Parallel Sum
	for (int NeighborOffsetSum = 1; NeighborOffsetSum < THREADGROUP_SIZE_1D; NeighborOffsetSum = (NeighborOffsetSum << 1))
	{
		int NeighborIndex = LinearThreadIndex + NeighborOffsetSum;
		float NeighborMajorant = (NeighborIndex < THREADGROUP_SIZE_1D) ? GSMajorant[NeighborIndex] : 0.0;
		GSMajorant[LinearThreadIndex] = max(GSMajorant[LinearThreadIndex], NeighborMajorant);

		GroupMemoryBarrierWithGroupSync();
	}

	int3 OutputDimensions = int3((InputDimensions + 3) / 4);
	uint LinearOuputIndex = GetLinearIndex(GroupId, OutputDimensions);

	if (LinearThreadIndex == 0)
	{
		FMajorantData MajorantData;
		MajorantData.Majorant = GSMajorant[0];
		SetMajorantData(RWOutputBuffer[LinearOuputIndex], MajorantData);
	}
}

[numthreads(1, 1, 1)]
void CopyMaximumIntoMajorantVoxelGridCS()
{
	SetMajorantData(RWMajorantVoxelGridBuffer[0], GetMajorantData(InputBuffer[0]));
}