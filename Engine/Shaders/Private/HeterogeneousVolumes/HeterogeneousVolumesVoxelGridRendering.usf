// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesVoxelGridTypes.ush"

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../ComputeShaderUtils.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesVoxelGridTraversal.ush"
#include "HeterogeneousVolumesVoxelGridRenderingUtils.ush"

#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"

#define DEBUG_MODE_DDA_MARCHING 0

#define DEBUG_MODE_SAMPLE_BASED_MARCHING 1
#define DEBUG_MODE_SAMPLE_BASED_MAJORANT 2
#define DEBUG_MODE_SAMPLE_BASED_MAX_MAJORANT 3
#define DEBUG_MODE_SAMPLE_BASED_MAJORANT_DDA 4

#define DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING 5
#define DEBUG_MODE_TRACKING_BASED_RATIO_TRACKING 6
#define DEBUG_MODE_TRACKING_BASED_SPECTRAL_RATIO_TRACKING 7
#define DEBUG_MODE_TRACKING_BASED_MAJORANT_DDA 8

#define DEBUG_MODE_SAMPLING_BASED \
	(DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MARCHING) || \
	(DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAJORANT) || \
	(DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAX_MAJORANT) || \
	(DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAJORANT_DDA)

#define DEBUG_MODE_TRACKING_BASED \
	(DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING) || \
	(DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_RATIO_TRACKING) || \
	(DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_SPECTRAL_RATIO_TRACKING) || \
	(DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_MAJORANT_DDA)

#define COLLISION_MODE_RATIO_TRACKING 0
#define COLLISION_MODE_MAX_COMPONENT 1
#define COLLISION_MODE_AVG_COMPONENT 2

struct FRenderDebugData
{
	float3 Estimate;
	float3 RayOrigin;
	float3 RayDirection;
	float TMin;
	float TMax;
	float Pdf;
	float Distance;

	float4x4 ScreenToTranslatedWorld;
	float4x4 ClipToTranslatedWorld;
};

// Ray data
int bJitter;
float MaxTraceDistance;
float StepSize;
int MaxStepCount;

// Dispatch data
int3 GroupCount;
int DebugMode;

// Output
RWTexture2D<float4> RWLightingTexture;
RWStructuredBuffer<FRenderDebugData> RWDebugBuffer;

struct FTrackingResult
{
	float3 Radiance;
	float3 Throughput;
	float Pdf;
	float Distance;
	bool bIsScattering;
};

FTrackingResult CreateTrackingResult(float Distance)
{
	FTrackingResult Result = (FTrackingResult)0;

	Result.Radiance = 0;
	Result.Throughput = 1;
	Result.Pdf = 0;
	Result.Distance = Distance;
	Result.bIsScattering = false;

	return Result;
}

// Concatenates the result of rhs onto lhs
FTrackingResult ConcatenateTrackingResult(FTrackingResult lhs, FTrackingResult rhs)
{
	FTrackingResult Result;
	//Result.Radiance = lhs.Radiance + rhs.Radiance * lhs.Throughput;
	Result.Radiance = rhs.Radiance * lhs.Throughput;
	Result.Throughput = lhs.Throughput * rhs.Throughput;
	Result.Pdf = lhs.Pdf * rhs.Pdf;
	Result.Distance = lhs.Distance + rhs.Distance;
	Result.bIsScattering = rhs.bIsScattering;
	return Result;
}

FTrackingResult AddTrackingResult(FTrackingResult lhs, FTrackingResult rhs)
{
	FTrackingResult Sum;
	Sum.Radiance = lhs.Radiance + rhs.Radiance;
	Sum.Throughput = lhs.Throughput + rhs.Throughput;
	Sum.Pdf = lhs.Pdf + rhs.Pdf;
	Sum.Distance = lhs.Distance + rhs.Distance;
	return Sum;
}

FTrackingResult DivideTrackingResult(FTrackingResult Value, int SPP)
{
	FTrackingResult Quotient;
	Quotient.Radiance = Value.Radiance / SPP;
	Quotient.Throughput = Value.Throughput / SPP;
	Quotient.Pdf = Value.Pdf / SPP;
	Quotient.Distance = Value.Distance / SPP;
	return Quotient;
}

float3 GetTranslatedWorldPos(float3 WorldPos)
{
	float3 TranslatedWorldPos = DFFastToTranslatedWorld(WorldPos, PrimaryView.PreViewTranslation);
	return TranslatedWorldPos;
}

FMajorantData EvalMajorant(float3 TranslatedWorldPos)
{
	FMajorantData MajorantData = CreateMajorantData();

	float3 TranslatedWorldBoundsMin = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin);
	float3 TranslatedWorldBoundsMax = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax);
	float3 TranslatedWorldBoundsExtent = TranslatedWorldBoundsMax - TranslatedWorldBoundsMin;
	float3 GridUV = (TranslatedWorldPos - TranslatedWorldBoundsMin) / TranslatedWorldBoundsExtent;
	if (all(GridUV >= 0.0) && all(GridUV <= 1.0))
	{
		float3 TopLevelVoxelPos = GridUV * OrthoGridUniformBuffer.TopLevelGridResolution;
		uint LinearTopLevelVoxelPos = GetLinearIndex(TopLevelVoxelPos, OrthoGridUniformBuffer.TopLevelGridResolution);

		UpdateMajorantData(MajorantData, GetMajorantData(OrthoGridUniformBuffer.MajorantGridBuffer[LinearTopLevelVoxelPos]));
	}

	return MajorantData;
}

float CalcStepSizeLocal(float3 WorldDirection)
{
	float3 TranslatedWorldBoundsMin = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin);
	float3 TranslatedWorldBoundsMax = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax);
	float3 TranslatedWorldBoundsExtent = TranslatedWorldBoundsMax - TranslatedWorldBoundsMin;
	float3 BottomLevelVoxelResolution = OrthoGridUniformBuffer.TopLevelGridResolution * 4;
	// Conditionally apply indirection dimensions
	BottomLevelVoxelResolution *= 4;

	float3 VoxelSize = TranslatedWorldBoundsExtent / BottomLevelVoxelResolution;
	float3 VoxelHitT = VoxelSize / abs(WorldDirection);
	float StepSize = min(VoxelHitT.x, min(VoxelHitT.y, VoxelHitT.z));
	return StepSize;
}

FTrackingResult SampleBasedMarch(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float WorldStepSize)
{
	FTrackingResult Result = CreateTrackingResult(TMin);

	float HitT = TMin;
	const float Epsilon = 1.0e-4;
	while (HitT < TMax && any(Result.Throughput > Epsilon))
	{
		float3 WorldPosition = WorldRayOrigin + WorldRayDirection * HitT;

		int LinearVoxelPos = -1;
		float3 Extinction = EvalExtinctionOutOfFrustum(WorldPosition, LinearVoxelPos);
		if (any(Extinction > 0))
		{
			Result.Radiance += GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearVoxelPos]) * Result.Throughput;
		}
		Result.Throughput *= exp(-Extinction * WorldStepSize);

		HitT += WorldStepSize;
	}
	Result.Distance = HitT;

	return Result;
}

FTrackingResult SampleBasedMajorantMarch(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float WorldStepSize)
{
	FTrackingResult Result = (FTrackingResult)0;
	Result.Throughput = 1.0;

	float HitT = TMin;
	const float Epsilon = 1.0e-3;
	while (HitT < TMax && any(Result.Throughput > Epsilon))
	{
		float3 WorldPosition = WorldRayOrigin + WorldRayDirection * HitT;

		FMajorantData MajorantData = EvalMajorant(WorldPosition);
		float3 Extinction = MajorantData.Majorant;
		Result.Throughput *= exp(-Extinction * WorldStepSize);

		HitT += WorldStepSize;
	}
	Result.Distance = HitT;
	Result.Pdf = Result.Throughput.x; // TODO: handle RGB?

	return Result;
}

FTrackingResult SampleBasedMaxMajorantMarch(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float WorldStepSize)
{
	FTrackingResult Result = (FTrackingResult)0;
	Result.Throughput = 1.0;

	float HitT = TMin;
	const float Epsilon = 1.0e-3;
	while (HitT < TMax && any(Result.Throughput > Epsilon))
	{
		float3 WorldPosition = WorldRayOrigin + WorldRayDirection * HitT;

		FMajorantData MajorantData = GetMajorantData(OrthoGridUniformBuffer.MajorantGridBuffer[0]);
		float3 Extinction = MajorantData.Majorant;
		Result.Throughput *= exp(-Extinction * WorldStepSize);

		HitT += WorldStepSize;
	}
	Result.Distance = HitT;
	Result.Pdf = Result.Throughput.x; // TODO: handle RGB?

	return Result;
}

float MaxComponent(float3 Value)
{
	return max(Value.x, max(Value.y, Value.z));
}

FTrackingResult DeltaTracking(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float SigmaBar, inout FRandomSequence RandSequence)
{
	FTrackingResult Result = CreateTrackingResult(TMin);

	float HitT = 0;
	while (Result.Distance < TMax)
	{
		// Pick a distance, based on SigmaBar
		float RandValue = RandSequence.Get1D();
		HitT = -log(1 - RandValue) / SigmaBar;
		Result.Distance += HitT;

		// Evaluate SigmaT at that location
		if (Result.Distance < TMax)
		{
			float3 WorldPosition = WorldRayOrigin + WorldRayDirection * Result.Distance;
			int LinearVoxelPos = -1;
			float3 Extinction = EvalExtinctionOutOfFrustum(WorldPosition, LinearVoxelPos);
			float SigmaT = Luminance(Extinction);

			//float AcceptanceProbability = MaxComponent(SigmaT) / SigmaBar;
			float AcceptanceProbability = min(SigmaT / SigmaBar, 1);
			if (RandSequence.Get1D() < AcceptanceProbability)
			{
				Result.Throughput = 0;
				Result.Radiance = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearVoxelPos]) / Luminance(SigmaT);
				break;
			}
		}
	}
	Result.Distance = min(Result.Distance, TMax);
	Result.Pdf = SigmaBar * exp(-SigmaBar * (Result.Distance - TMin));

	return Result;
}

FTrackingResult SpectralWeightedDeltaTracking(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float SigmaBar, int CollisionMode, inout FRandomSequence RandSequence)
{
	FTrackingResult Result = CreateTrackingResult(TMin);

	float HitT = 0;
	const float Epsilon = 1.0e-4;
	while (Result.Distance < TMax && all(Result.Throughput > Epsilon))
	{
		// Distance-sampling, based on SigmaBar
		float RandValue = RandSequence.Get1D();
		HitT = -log(1 - RandValue) / SigmaBar;
		Result.Distance += HitT;

		if (Result.Distance < TMax)
		{
			float3 WorldPosition = WorldRayOrigin + WorldRayDirection * Result.Distance;
			int LinearVoxelPos = -1;
			float3 Extinction = EvalExtinctionOutOfFrustum(WorldPosition, LinearVoxelPos);

			// Pick a volume coefficient, based on proportional weight
			float3 Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearVoxelPos]) / Extinction;
			float3 SpectralAbsorptionCollisionWeight = Extinction * (1 - Albedo) / SigmaBar;
			float3 SpectralScatteringCollisionWeight = Extinction * Albedo / SigmaBar;
			float3 SpectralNullCollisionWeight = 1.0 - Extinction / SigmaBar;

			// Set collision probabilities
			float AbsorptionProbability = 0;
			float ScatteringProbability = 0;
			float NullCollisionProbability = 0;
			if (CollisionMode == COLLISION_MODE_MAX_COMPONENT)
			{
				AbsorptionProbability = MaxComponent(SpectralAbsorptionCollisionWeight);
				ScatteringProbability = MaxComponent(SpectralScatteringCollisionWeight);
				NullCollisionProbability = MaxComponent(SpectralNullCollisionWeight);
			}
			else if (CollisionMode == COLLISION_MODE_AVG_COMPONENT)
			{
				AbsorptionProbability = dot(SpectralAbsorptionCollisionWeight, Result.Throughput);
				ScatteringProbability = dot(SpectralScatteringCollisionWeight, Result.Throughput);
				NullCollisionProbability = dot(SpectralNullCollisionWeight, Result.Throughput);
			}
			else if (CollisionMode == COLLISION_MODE_RATIO_TRACKING)
			{
				AbsorptionProbability = 0;
				ScatteringProbability = 0;
				NullCollisionProbability = 1;
			}

			// Collision-sampling
			float3 CollisionCdf = float3(AbsorptionProbability,
				AbsorptionProbability + ScatteringProbability,
				AbsorptionProbability + ScatteringProbability + NullCollisionProbability
			);
			float CollisionSample = RandSequence.Get1D() * CollisionCdf.z;

			// Normalize probabilities
			AbsorptionProbability *= rcp(CollisionCdf.z);
			ScatteringProbability *= rcp(CollisionCdf.z);
			NullCollisionProbability *= rcp(CollisionCdf.z);

			// Emission
			if (CollisionSample < CollisionCdf.x)
			{
				Result.Throughput *= SpectralAbsorptionCollisionWeight / AbsorptionProbability;
				Result.Radiance = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearVoxelPos]) * Result.Throughput;
				Result.Throughput = 0;
				break;
			}
			// Scattering
			else if (CollisionSample < CollisionCdf.y)
			{
				// Compute in-scattering
				Result.Throughput *= SpectralScatteringCollisionWeight / ScatteringProbability;
				Result.bIsScattering = true;
				break;
			}
			// Transmittance
			else
			{
				Result.Throughput *= SpectralNullCollisionWeight / NullCollisionProbability;
			}
		}
	}
	Result.Distance = min(Result.Distance, TMax);
	Result.Pdf = SigmaBar * exp(-SigmaBar * (Result.Distance - TMin));

	return Result;
}

FTrackingResult RatioTracking(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float SigmaBar, inout FRandomSequence RandSequence)
{
	FTrackingResult Result = CreateTrackingResult(TMin);

	const float Epsilon = 1.0e-5;
	float HitT = 0;
	while (Result.Distance < TMax && all(Result.Throughput > Epsilon))
	{
		// Pick a distance, based on SigmaBar
		float RandValue = RandSequence.Get1D();
		HitT = -log(1 - RandValue) / SigmaBar;
		Result.Distance += HitT;

		// Evaluate SigmaT at that location
		if (Result.Distance < TMax)
		{
			float3 WorldPosition = WorldRayOrigin + WorldRayDirection * Result.Distance;
			int LinearVoxelPos = -1;
			float3 Extinction = EvalExtinctionOutOfFrustum(WorldPosition, LinearVoxelPos);
			float SigmaT = Luminance(Extinction);

			float AcceptanceProbability = min(SigmaT / SigmaBar, 1);
			if (RandSequence.Get1D() < AcceptanceProbability)
			{
				//Result.Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearVoxelPos]) / Luminance(SigmaT);
			}
			Result.Throughput *= (1 - AcceptanceProbability);
		}
	}
	Result.Distance = min(Result.Distance, TMax);
	Result.Pdf = SigmaBar * exp(-SigmaBar * (Result.Distance - TMin));

	return Result;
}

FTrackingResult SpectralRatioTracking(float3 WorldRayOrigin, float3 WorldRayDirection, float TMin, float TMax, float SigmaBar, inout FRandomSequence RandSequence)
{
	FTrackingResult Result = CreateTrackingResult(TMin);

	const float Epsilon = 1.0e-5;
	float HitT = 0;
	while (Result.Distance < TMax && all(Result.Throughput > Epsilon))
	{
		// Pick a distance, based on SigmaBar
		float RandValue = RandSequence.Get1D();
		HitT = -log(1 - RandValue) / SigmaBar;
		Result.Distance += HitT;

		// Evaluate SigmaT at that location
		if (Result.Distance < TMax)
		{
			float3 WorldPosition = WorldRayOrigin + WorldRayDirection * Result.Distance;
			int LinearVoxelPos = -1;
			float3 Extinction = EvalExtinctionOutOfFrustum(WorldPosition, LinearVoxelPos);

			// Pick a component, based on proportional weight
			float3 SpectralCdf = float3(Extinction.x, Extinction.x + Extinction.y, Extinction.x + Extinction.y + Extinction.z);
			float RandSpectra = RandSequence.Get1D() * SpectralCdf.z;
			float SigmaT = (RandSpectra < SpectralCdf.x) ? Extinction.x : ((RandSpectra < SpectralCdf.y) ? Extinction.y : Extinction.z);

			if (SigmaT > 0)
			{
				float3 SpectralWeight = Extinction / SigmaT;
				float AcceptanceProbability = min(SigmaT / SigmaBar, 1);
				float3 SpectralAcceptanceProbability = min(SpectralWeight * SigmaT / SigmaBar, 1);
				if (RandSequence.Get1D() < AcceptanceProbability)
				{
					//Result.Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearVoxelPos]) / Luminance(SigmaT);
				}
				Result.Throughput *= (1 - SpectralAcceptanceProbability);
			}
		}
	}
	Result.Distance = min(Result.Distance, TMax);
	Result.Pdf = SigmaBar * exp(-SigmaBar * (Result.Distance - TMin));

	return Result;
}

FTrackingResult DDABasedMajorantMarch(float3 WorldRayOrigin, float3 Direction, float TMin, float TMax, int DistanceSamplingMethod, int CollisionSamplingMethod, inout FRandomSequence RandSequence)
{
	FTrackingResult TrackingResult = CreateTrackingResult(TMin);

	float3 WorldRayBegin = WorldRayOrigin + Direction * TMin;
	float3 WorldRayEnd = WorldRayOrigin + Direction * TMax;
	float WorldRayTMax = length(WorldRayEnd - WorldRayBegin);

	// Transform to voxel-space
	float3 WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	float3 TopLevelGridWorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 VoxelRayBegin = (WorldRayBegin - WorldBoundsMin) / TopLevelGridWorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayEnd = (WorldRayEnd - WorldBoundsMin) / TopLevelGridWorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayDirection = VoxelRayEnd - VoxelRayBegin;
	float VoxelRayTMin = 0.0;
	float VoxelRayTMax = length(VoxelRayDirection);
	VoxelRayDirection /= VoxelRayTMax;

	float VoxelToWorldScale = WorldRayTMax / VoxelRayTMax;

	// March majorant grid via DDA
	float3 VoxelRayDirectionInv = 1.0 / VoxelRayDirection;

	float VoxelRayMarchT = VoxelRayTMin;
	float3 VoxelRayMarchPos = VoxelRayBegin;

	float3 VoxelBoundsPos;
	VoxelBoundsPos.x = sign(VoxelRayDirection.x) > 0 ? floor(VoxelRayMarchPos.x) + 1 : ceil(VoxelRayMarchPos.x) - 1;
	VoxelBoundsPos.y = sign(VoxelRayDirection.y) > 0 ? floor(VoxelRayMarchPos.y) + 1 : ceil(VoxelRayMarchPos.y) - 1;
	VoxelBoundsPos.z = sign(VoxelRayDirection.z) > 0 ? floor(VoxelRayMarchPos.z) + 1 : ceil(VoxelRayMarchPos.z) - 1;

	float3 VoxelBoundsHitT = (VoxelBoundsPos - VoxelRayMarchPos) * VoxelRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (VoxelBoundsHitT.x <= Epsilon) VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
	if (VoxelBoundsHitT.y <= Epsilon) VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
	if (VoxelBoundsHitT.z <= Epsilon) VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);

	while ((VoxelRayMarchT < VoxelRayTMax) && any(TrackingResult.Throughput > Epsilon) && !TrackingResult.bIsScattering)
	{
		float VoxelRayMarchDeltaT = 0.0;
		if (VoxelBoundsHitT.x < VoxelBoundsHitT.y)
		{
			if (VoxelBoundsHitT.x < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.x - VoxelRayMarchT;
				VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}
		else
		{
			if (VoxelBoundsHitT.y < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.y - VoxelRayMarchT;
				VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (VoxelRayMarchT + VoxelRayMarchDeltaT > VoxelRayTMax)
		{
			VoxelRayMarchDeltaT = VoxelRayTMax - VoxelRayMarchT;
		}

		// Sample at the midpoint of the voxel ray
		float3 VoxelEndPos = VoxelRayMarchPos + VoxelRayDirection * VoxelRayMarchDeltaT;
		float3 VoxelSamplePos = (VoxelRayMarchPos + VoxelEndPos) * 0.5;
		uint VoxelSampleLinearPos = GetLinearIndex(VoxelSamplePos, OrthoGridUniformBuffer.TopLevelGridResolution);
		FMajorantData MajorantData = GetMajorantData(OrthoGridUniformBuffer.MajorantGridBuffer[VoxelSampleLinearPos]);

		float3 TranslatedWorldPos = WorldRayBegin + Direction * VoxelRayMarchT * VoxelToWorldScale;
		if (DistanceSamplingMethod == DEBUG_MODE_SAMPLE_BASED_MARCHING)
		{
			float StepSize = CalcStepSizeLocal(Direction);
			FTrackingResult Sample = SampleBasedMarch(TranslatedWorldPos, Direction, 0.0, VoxelRayMarchDeltaT * VoxelToWorldScale, StepSize);
			TrackingResult = ConcatenateTrackingResult(TrackingResult, Sample);
		}
		else if (DistanceSamplingMethod == DEBUG_MODE_TRACKING_BASED_SPECTRAL_RATIO_TRACKING)
		{
			FTrackingResult Sample = SpectralRatioTracking(TranslatedWorldPos, Direction, 0.0, VoxelRayMarchDeltaT * VoxelToWorldScale, MajorantData.Majorant, RandSequence);
			TrackingResult = ConcatenateTrackingResult(TrackingResult, Sample);
		}
		else if (DistanceSamplingMethod == DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING)
		{
			//FTrackingResult Sample = SpectralWeightedDeltaTracking(TranslatedWorldPos, Direction, 0.0, VoxelRayMarchDeltaT * VoxelToWorldScale, MajorantData.Majorant, CollisionSamplingMethod, RandSequence);
			FTrackingResult Sample = DeltaTracking(TranslatedWorldPos, Direction, 0.0, VoxelRayMarchDeltaT * VoxelToWorldScale, MajorantData.Majorant, RandSequence);
			TrackingResult = ConcatenateTrackingResult(TrackingResult, Sample);
		}
		else
		{
			FTrackingResult Sample = CreateTrackingResult(0);
			Sample.Distance += VoxelRayMarchDeltaT * VoxelToWorldScale;
			Sample.Throughput *= exp(-MajorantData.Majorant * Sample.Distance);
			TrackingResult = ConcatenateTrackingResult(TrackingResult, Sample);
		}

		VoxelRayMarchT += VoxelRayMarchDeltaT;
		VoxelRayMarchPos = VoxelEndPos;
	}

	return TrackingResult;
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderTransmittanceWithVoxelGridCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID
)
{
	float3 Radiance = 0.0;

	// Create screen ray
	if (any(DispatchThreadId.xy >= View.ViewSizeAndInvSize.xy))
	{
		return;
	}
	uint LinearIndex = DispatchThreadId.y * GroupCount.x * THREADGROUP_SIZE_2D + DispatchThreadId.x;
	uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy;

	// Extract depth
	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	DeviceZ = max(0.000000000001, DeviceZ);

	// Clip trace distance
	float SceneDepth = min(ConvertFromDeviceZ(DeviceZ), MaxTraceDistance);
	DeviceZ = ConvertToDeviceZ(SceneDepth);

	float3 RayOrigin = GetTranslatedWorldCameraPosFromView(PixelCoord + 0.5);
	float3 RayEnd = SvPositionToTranslatedWorld(float4(PixelCoord + 0.5, DeviceZ, 1));
	float3 RayDirection = RayEnd - RayOrigin;
	float RayTMin = 0.0;
	float RayTMax = length(RayDirection);
	RayDirection /= RayTMax;
	
	FRandomSequence RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex);
	float Jitter = RandSequence.Get1D();

	float3 Emission = 0;
	float3 Transmittance = 1;

	float3 BoundsMin = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin);
	float3 BoundsMax = GetTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax);
	float2 ClipHitT = IntersectAABB(RayOrigin, RayDirection, RayTMin, RayTMax, BoundsMin, BoundsMax);

#if DEBUG_MODE_SAMPLING_BASED
	if (ClipHitT.x < ClipHitT.y)
	{
		float StepSize = CalcStepSizeLocal(RayDirection);
		Jitter *= StepSize;
#if DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MARCHING
		FTrackingResult Result = SampleBasedMarch(RayOrigin, RayDirection, ClipHitT.x + Jitter, ClipHitT.y, StepSize);
#elif DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAJORANT
		FTrackingResult Result = SampleBasedMajorantMarch(RayOrigin, RayDirection, ClipHitT.x + Jitter, ClipHitT.y, StepSize);
#elif DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAX_MAJORANT
		FTrackingResult Result = SampleBasedMaxMajorantMarch(RayOrigin, RayDirection, ClipHitT.x + Jitter, ClipHitT.y, StepSize);
#else // DEBUG_MODE == DEBUG_MODE_SAMPLE_BASED_MAJORANT_DDA
		int DistanceSamplingMethod = DEBUG_MODE_SAMPLE_BASED_MARCHING;
		int CollisionSamplingMethod = 0;
		FTrackingResult Result = DDABasedMajorantMarch(RayOrigin, RayDirection, ClipHitT.x, ClipHitT.y, DistanceSamplingMethod, CollisionSamplingMethod, RandSequence);
#endif
		Transmittance = Result.Throughput;
	}

	//Radiance = Result.Emission;
	Radiance = 1.0 - Transmittance;

#elif DEBUG_MODE_TRACKING_BASED
	if (ClipHitT.x < ClipHitT.y)
	{
		FTrackingResult Result = (FTrackingResult) 0;

		FMajorantData MajorantData = GetMajorantData(OrthoGridUniformBuffer.MajorantGridBuffer[0]);
		float SigmaBar = MajorantData.Majorant;

		int SPPCount = 1;
		for (int Index = 0; Index < SPPCount; ++Index)
		{
			RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex * SPPCount + Index);
#if DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING
			FTrackingResult Sample = SpectralWeightedDeltaTracking(RayOrigin, RayDirection, ClipHitT.x, ClipHitT.y, SigmaBar, COLLISION_MODE_AVG_COMPONENT, RandSequence);
#elif DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_RATIO_TRACKING
			FTrackingResult Sample = RatioTracking(RayOrigin, RayDirection, ClipHitT.x, ClipHitT.y, SigmaBar, RandSequence);
#elif DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_SPECTRAL_RATIO_TRACKING
			FTrackingResult Sample = SpectralRatioTracking(RayOrigin, RayDirection, ClipHitT.x, ClipHitT.y, SigmaBar, RandSequence);
#else // DEBUG_MODE == DEBUG_MODE_TRACKING_BASED_MAJORANT_DDA
			//int DistanceSamplingMethod = DEBUG_MODE_TRACKING_BASED_SPECTRAL_RATIO_TRACKING;
			int DistanceSamplingMethod = DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING;
			int CollisionSamplingMethod = COLLISION_MODE_AVG_COMPONENT;
			FTrackingResult Sample = DDABasedMajorantMarch(RayOrigin, RayDirection, ClipHitT.x, ClipHitT.y, DistanceSamplingMethod, CollisionSamplingMethod, RandSequence);
#endif
#if 0
			// Faux direct-lighting ray
			if (Sample.bIsScattering)
			{
				float3 ShadowRayOrigin = RayOrigin + RayDirection * Sample.Distance;
				float3 LightDirection = normalize(float3(0, 0, 1));
				float2 ClipShadowHitT = IntersectAABB(ShadowRayOrigin, LightDirection, 0, 10000, BoundsMin, BoundsMax);
				if (ClipShadowHitT.y > ClipShadowHitT.x)
				{
					float3 Le = 10000;
					float Phase = 1.0 / (4 * PI);
					FTrackingResult Lighting = DDABasedMajorantMarch(ShadowRayOrigin, LightDirection, ClipShadowHitT.x, ClipShadowHitT.y, DEBUG_MODE_TRACKING_BASED_DELTA_TRACKING, COLLISION_MODE_RATIO_TRACKING, RandSequence);
					Sample.Radiance = Le * Phase * Lighting.Throughput * Sample.Throughput;// / (Sample.Pdf * Lighting.Pdf);
				}
				Sample.Throughput = 0;
			}
#endif
			Result = AddTrackingResult(Result, Sample);
		}

		Result = DivideTrackingResult(Result, SPPCount);
		float3 Le = 1;
		Radiance = (Result.Radiance + (1.0 - Result.Throughput) * Le);
		Transmittance = Result.Throughput;
	}

#elif DEBUG_MODE == DEBUG_MODE_DDA_MARCHING
	float FrustumRayTMin = RayTMin;
	float FrustumRayTMax = RayTMax;
	if (FrustumGridUniformBuffer.bUseFrustumGrid)
	{
		float ViewNearZ = min(ConvertFromDeviceZ(DeviceZ), FrustumGridUniformBuffer.NearPlaneDepth);
		float DeviceNearZ = ConvertToDeviceZ(ViewNearZ);
		float ViewFarZ = min(ConvertFromDeviceZ(DeviceZ), FrustumGridUniformBuffer.FarPlaneDepth);
		float DeviceFarZ = ConvertToDeviceZ(ViewFarZ);

		// TODO: Shared code is imprecise, in comparison with commented out frustum-trimmed ray. Investigate...
#if 0
		float3 RayBegin = SvPositionToTranslatedWorld(float4(PixelCoord + 0.5, DeviceNearZ, 1));
		float3 RayEnd = SvPositionToTranslatedWorld(float4(PixelCoord + 0.5, DeviceFarZ, 1));
		float3 RayDirection = RayEnd - RayBegin;
		FrustumRayTMin = 0.0;
		FrustumRayTMax = length(RayDirection);
		RayDirection /= FrustumRayTMax;

		Transmittance = TraceFrustumVoxelGrid(RayBegin, RayDirection, FrustumRayTMin, FrustumRayTMax);
		float RayTOffset = length(RayBegin - RayOrigin);
		FrustumRayTMin += RayTOffset;
		FrustumRayTMax += RayTOffset;
#else
		float3 WorldRayOrigin = DFHackToFloat(SvPositionToWorld(float4(PixelCoord + 0.5, DeviceNearZ, 1)));
		float3 WorldRayEnd = DFHackToFloat(SvPositionToWorld(float4(PixelCoord + 0.5, DeviceFarZ, 1)));
		float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
		float WorldRayLength = length(WorldRayDirection);
		WorldRayDirection /= WorldRayLength;

		// Convert to view-space
		float3 ViewRayOrigin = mul(float4(WorldRayOrigin, 1), FrustumGridUniformBuffer.WorldToView).xyz;
		float3 ViewRayEnd = mul(float4(WorldRayEnd, 1), FrustumGridUniformBuffer.WorldToView).xyz;
		float3 ViewRayDirection = ViewRayEnd - ViewRayOrigin;
		float ViewRayLength = length(ViewRayDirection);
		ViewRayDirection /= ViewRayLength;

		// Convert to voxel-space
		int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
		float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
		float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
		float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
		float3 VoxelRayOrigin = ViewToVoxel(ViewRayOrigin, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);
		float3 VoxelRayEnd = ViewToVoxel(ViewRayOrigin + ViewRayDirection * ViewRayLength, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

		float3 VoxelRayDirection = VoxelRayEnd - VoxelRayOrigin;
		float VoxelRayLength = length(VoxelRayDirection);
		VoxelRayDirection /= VoxelRayLength;

		float VoxelRayTMin = 0.0;
		float VoxelRayTMax = VoxelRayLength;

		// Traversal
		{
			MarchTopLevelFrustumVoxelGrid(
				VoxelRayOrigin,
				VoxelRayDirection,
				VoxelRayTMin,
				VoxelRayTMax,
				WorldRayLength,
				FrustumGridUniformBuffer.TopLevelFroxelGridResolution,
				FrustumGridUniformBuffer.TopLevelFroxelGridBuffer,
				FrustumGridUniformBuffer.ExtinctionFroxelGridBuffer,
				Transmittance
			);
		}

		float3 RayBegin = SvPositionToTranslatedWorld(float4(PixelCoord + 0.5, DeviceNearZ, 1));
		float RayTOffset = length(RayBegin - RayOrigin);
		FrustumRayTMin += RayTOffset;
		FrustumRayTMax += RayTOffset;
#endif
	}

	if (OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		if (FrustumGridUniformBuffer.bUseFrustumGrid && (FrustumRayTMin < FrustumRayTMax))
		{
			Transmittance *= TraceOrthoVoxelGrid(RayOrigin, RayDirection, RayTMin, FrustumRayTMin);
			Transmittance *= TraceOrthoVoxelGrid(RayOrigin, RayDirection, FrustumRayTMax, RayTMax);
		}
		else
		{
			Transmittance = TraceOrthoVoxelGrid(RayOrigin, RayDirection, RayTMin, RayTMax);
		}
	}

#endif // DEBUG_MODE_DDA_MAJORANT

	// Output..
	RWLightingTexture[PixelCoord] += float4(Radiance, Luminance(Transmittance));
}
