// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesVoxelGridUtils.ush"
#include "HeterogeneousVolumesFrustumVoxelGridUtils.ush"

float3 EvalExtinctionOutOfFrustum(float3 WorldPos, inout int LinearBottomLevelVoxelPos)
{
	float3 Extinction = 0;

	float3 WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	float3 WorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 GridUV = (WorldPos - WorldBoundsMin) / WorldBoundsExtent;
	if (all(GridUV >= 0.0) && all(GridUV <= 1.0))
	{
		float3 TopLevelVoxelPos = GridUV * OrthoGridUniformBuffer.TopLevelGridResolution;
		uint LinearTopLevelVoxelPos = GetLinearIndex(TopLevelVoxelPos, OrthoGridUniformBuffer.TopLevelGridResolution);

		if (OrthoGridUniformBuffer.bEnableIndirectionGrid)
		{
			FTopLevelGridData TopLevelGridData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelGridData))
			{
				uint IndirectionGridIndex = GetBottomLevelIndex(TopLevelGridData);
				uint3 IndirectionGridVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

				// Convert to Indirection voxel-space
				float3 IndirectionGridVoxelPos = frac(TopLevelVoxelPos) * IndirectionGridVoxelResolution;
				int3 IndirectionGridVoxelPosAsInt = clamp(IndirectionGridVoxelPos, 0, IndirectionGridVoxelResolution - 1);
				FTopLevelGridData IndirectionData = OrthoGridUniformBuffer.IndirectionGridBuffer[IndirectionGridIndex + MortonEncode3(IndirectionGridVoxelPosAsInt)];
				if (IsBottomLevelAllocated(IndirectionData))
				{
					uint BottomLevelIndex = GetBottomLevelIndex(IndirectionData);
					uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(IndirectionData);

					// Constant interpolation
					float3 BottomLevelVoxelPos = frac(IndirectionGridVoxelPos) * BottomLevelVoxelResolution;
					int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
					LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);
					Extinction = GetExtinction(OrthoGridUniformBuffer.ExtinctionGridBuffer[LinearBottomLevelVoxelPos]);
					//Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
				}
			}
		}
		else
		{
			FTopLevelGridData TopLevelData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelData))
			{
				uint BottomLevelIndex = GetBottomLevelIndex(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);
				uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);

				// Constant Interpolation
				float3 BottomLevelVoxelPos = frac(TopLevelVoxelPos) * BottomLevelVoxelResolution;
				int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
				LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);

				Extinction = GetExtinction(OrthoGridUniformBuffer.ExtinctionGridBuffer[LinearBottomLevelVoxelPos]);
				//Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
			}
		}
	}

	return Extinction;
}

float3 EvalExtinctionInFrustum(float3 WorldPos, inout bool bInFrustum)
{
	float3 Extinction = 0;

	// Convert WorldPos to voxel space
	float3 ViewPos = mul(float4(WorldPos, 1), FrustumGridUniformBuffer.WorldToView).xyz;

	int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
	float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
	float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
	float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
	float3 VoxelPos = ViewToVoxel(ViewPos, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

	bInFrustum = all(VoxelPos > 0) && all(VoxelPos < FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
	if (bInFrustum)
	{
		uint LinearTopLevelVoxelPos = GetLinearIndex(VoxelPos, FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
		FTopLevelGridData TopLevelGridData = FrustumGridUniformBuffer.TopLevelFroxelGridBuffer[LinearTopLevelVoxelPos];
		if (IsBottomLevelAllocated(TopLevelGridData))
		{
			uint BottomLevelIndex = GetBottomLevelIndex(TopLevelGridData);
			uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

			float3 BottomLevelVoxelPos = frac(VoxelPos) * BottomLevelVoxelResolution;
			uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(uint3(BottomLevelVoxelPos));
			Extinction = GetExtinction(FrustumGridUniformBuffer.ExtinctionFroxelGridBuffer[LinearBottomLevelVoxelPos]);
		}
	}

	return Extinction;
}

float3 EvalExtinction(float3 WorldPos)
{
	bool bInFrustum = false;
	float3 Extinction = EvalExtinctionInFrustum(WorldPos, bInFrustum);
	if (!bInFrustum)
	{
		int LinearVoxelPos;
		Extinction = EvalExtinctionOutOfFrustum(WorldPos, LinearVoxelPos);
	}
	return Extinction;
}

float3 EvalAlbedoOutOfFrustum(float3 WorldPos, inout int LinearBottomLevelVoxelPos)
{
	float3 Albedo = 0;

	float3 WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	float3 WorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 GridUV = (WorldPos - WorldBoundsMin) / WorldBoundsExtent;
	if (all(GridUV >= 0.0) && all(GridUV <= 1.0))
	{
		float3 TopLevelVoxelPos = GridUV * OrthoGridUniformBuffer.TopLevelGridResolution;
		uint LinearTopLevelVoxelPos = GetLinearIndex(TopLevelVoxelPos, OrthoGridUniformBuffer.TopLevelGridResolution);

		if (OrthoGridUniformBuffer.bEnableIndirectionGrid)
		{
			FTopLevelGridData TopLevelGridData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelGridData))
			{
				uint IndirectionGridIndex = GetBottomLevelIndex(TopLevelGridData);
				uint3 IndirectionGridVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

				// Convert to Indirection voxel-space
				float3 IndirectionGridVoxelPos = frac(TopLevelVoxelPos) * IndirectionGridVoxelResolution;
				int3 IndirectionGridVoxelPosAsInt = clamp(IndirectionGridVoxelPos, 0, IndirectionGridVoxelResolution - 1);
				FTopLevelGridData IndirectionData = OrthoGridUniformBuffer.IndirectionGridBuffer[IndirectionGridIndex + MortonEncode3(IndirectionGridVoxelPosAsInt)];
				if (IsBottomLevelAllocated(IndirectionData))
				{
					uint BottomLevelIndex = GetBottomLevelIndex(IndirectionData);
					uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(IndirectionData);

					// Constant interpolation
					float3 BottomLevelVoxelPos = frac(IndirectionGridVoxelPos) * BottomLevelVoxelResolution;
					int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
					LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);
					Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
					//Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
				}
			}
		}
		else
		{
			FTopLevelGridData TopLevelData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelData))
			{
				uint BottomLevelIndex = GetBottomLevelIndex(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);
				uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);

				// Constant Interpolation
				float3 BottomLevelVoxelPos = frac(TopLevelVoxelPos) * BottomLevelVoxelResolution;
				int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
				LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);

				Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
				//Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
			}
		}
	}

	return Albedo;
}

float3 EvalAlbedoInFrustum(float3 WorldPos, inout bool bInFrustum)
{
	float3 Albedo = 0;

	// Convert WorldPos to voxel space
	float3 ViewPos = mul(float4(WorldPos, 1), FrustumGridUniformBuffer.WorldToView).xyz;

	int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
	float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
	float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
	float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
	float3 VoxelPos = ViewToVoxel(ViewPos, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

	bInFrustum = all(VoxelPos > 0) && all(VoxelPos < FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
	if (bInFrustum)
	{
		uint LinearTopLevelVoxelPos = GetLinearIndex(VoxelPos, FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
		FTopLevelGridData TopLevelGridData = FrustumGridUniformBuffer.TopLevelFroxelGridBuffer[LinearTopLevelVoxelPos];
		if (IsBottomLevelAllocated(TopLevelGridData))
		{
			uint BottomLevelIndex = GetBottomLevelIndex(TopLevelGridData);
			uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

			float3 BottomLevelVoxelPos = frac(VoxelPos) * BottomLevelVoxelResolution;
			uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(uint3(BottomLevelVoxelPos));
			Albedo = GetAlbedo(FrustumGridUniformBuffer.ScatteringFroxelGridBuffer[LinearBottomLevelVoxelPos]);
		}
	}

	return Albedo;
}

float3 EvalAlbedo(float3 WorldPos)
{
	bool bInFrustum = false;
	float3 Albedo = EvalAlbedoInFrustum(WorldPos, bInFrustum);
	if (!bInFrustum)
	{
		int LinearVoxelPos;
		Albedo = EvalAlbedoOutOfFrustum(WorldPos, LinearVoxelPos);
	}
	return Albedo;
}

float3 EvalEmissionOutOfFrustum(float3 WorldPos, inout int LinearBottomLevelVoxelPos)
{
	float3 Emission = 0;

	float3 WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	float3 WorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 GridUV = (WorldPos - WorldBoundsMin) / WorldBoundsExtent;
	if (all(GridUV >= 0.0) && all(GridUV <= 1.0))
	{
		float3 TopLevelVoxelPos = GridUV * OrthoGridUniformBuffer.TopLevelGridResolution;
		uint LinearTopLevelVoxelPos = GetLinearIndex(TopLevelVoxelPos, OrthoGridUniformBuffer.TopLevelGridResolution);

		if (OrthoGridUniformBuffer.bEnableIndirectionGrid)
		{
			FTopLevelGridData TopLevelGridData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelGridData))
			{
				uint IndirectionGridIndex = GetBottomLevelIndex(TopLevelGridData);
				uint3 IndirectionGridVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

				// Convert to Indirection voxel-space
				float3 IndirectionGridVoxelPos = frac(TopLevelVoxelPos) * IndirectionGridVoxelResolution;
				int3 IndirectionGridVoxelPosAsInt = clamp(IndirectionGridVoxelPos, 0, IndirectionGridVoxelResolution - 1);
				FTopLevelGridData IndirectionData = OrthoGridUniformBuffer.IndirectionGridBuffer[IndirectionGridIndex + MortonEncode3(IndirectionGridVoxelPosAsInt)];
				if (IsBottomLevelAllocated(IndirectionData))
				{
					uint BottomLevelIndex = GetBottomLevelIndex(IndirectionData);
					uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(IndirectionData);

					// Constant interpolation
					float3 BottomLevelVoxelPos = frac(IndirectionGridVoxelPos) * BottomLevelVoxelResolution;
					int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
					LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);
					Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearBottomLevelVoxelPos]);
				}
			}
		}
		else
		{
			FTopLevelGridData TopLevelData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelData))
			{
				uint BottomLevelIndex = GetBottomLevelIndex(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);
				uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);

				// Constant Interpolation
				float3 BottomLevelVoxelPos = frac(TopLevelVoxelPos) * BottomLevelVoxelResolution;
				int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
				LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);

				Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearBottomLevelVoxelPos]);
			}
		}
	}

	return Emission;
}

float3 EvalEmissionInFrustum(float3 WorldPos, inout bool bInFrustum)
{
	float3 Emission = 0;

	// Convert WorldPos to voxel space
	float3 ViewPos = mul(float4(WorldPos, 1), FrustumGridUniformBuffer.WorldToView).xyz;

	int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
	float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
	float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
	float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
	float3 VoxelPos = ViewToVoxel(ViewPos, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

	bInFrustum = all(VoxelPos > 0) && all(VoxelPos < FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
	if (bInFrustum)
	{
		uint LinearTopLevelVoxelPos = GetLinearIndex(VoxelPos, FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
		FTopLevelGridData TopLevelGridData = FrustumGridUniformBuffer.TopLevelFroxelGridBuffer[LinearTopLevelVoxelPos];
		if (IsBottomLevelAllocated(TopLevelGridData))
		{
			uint BottomLevelIndex = GetBottomLevelIndex(TopLevelGridData);
			uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

			float3 BottomLevelVoxelPos = frac(VoxelPos) * BottomLevelVoxelResolution;
			uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(uint3(BottomLevelVoxelPos));
			Emission = GetEmission(FrustumGridUniformBuffer.EmissionFroxelGridBuffer[LinearBottomLevelVoxelPos]);
		}
	}

	return Emission;
}

float3 EvalEmission(float3 WorldPos)
{
	bool bInFrustum = false;
	float3 Emission = EvalEmissionInFrustum(WorldPos, bInFrustum);
	if (!bInFrustum)
	{
		int LinearVoxelPos;
		Emission = EvalEmissionOutOfFrustum(WorldPos, LinearVoxelPos);
	}
	return Emission;
}