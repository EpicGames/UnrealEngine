// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumesVoxelGridTypes.ush"

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../ComputeShaderUtils.ush"
#include "HeterogeneousVolumesStochasticFiltering.ush"
#include "HeterogeneousVolumesTracingUtils.ush"
#include "HeterogeneousVolumesVoxelGridTraversal.ush"
#include "HeterogeneousVolumesVoxelGridRenderingUtils.ush"
#include "../BlueNoise.ush"

#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"

#ifndef USE_AVSM_COMPRESSION
#define USE_AVSM_COMPRESSION 0
#endif // USE_AVSM_COMPRESSION

// Ray data
int bJitter;
float ShadowStepSize;

// Dispatch data
int3 GroupCount;
int ShadowDebugTweak;

int MaxStepCount;
int StochasticFilteringMode;

int NumShadowMatrices;
float4x4 TranslatedWorldToShadow[6];
float4x4 ShadowToTranslatedWorld[6];
float3 TranslatedWorldOrigin;

// Lighting
int bApplyEmissionAndTransmittance;
int bApplyDirectLighting;
int bApplyShadowTransmittance;
int LightType;
float VolumetricScatteringIntensity;

// Beer shadow map
Texture2D<float4> BeerShadowMapTexture;
int2 ShadowResolution;

RWTexture2D<float4> RWBeerShadowMapTexture;

struct FVolumeSampleContext
{
	float3 WorldPosition;
};

float4x4 IdentityMatrix()
{
	return float4x4(
		1, 0, 0, 0,
		0, 1, 0, 0,
		0, 0, 1, 0,
		0, 0, 0, 1);
}

float4x4 GetLocalToWorld()
{
	return IdentityMatrix();
}

float4x4 GetWorldToLocal()
{
	return IdentityMatrix();
}

float3 GetLocalBoundsMin()
{
	float3 LocalBoundsMin = 0;
	if (OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		LocalBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	}
	else if (FrustumGridUniformBuffer.bUseFrustumGrid)
	{
		LocalBoundsMin = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMin;
	}
	return LocalBoundsMin;
}

float3 GetLocalBoundsMax()
{
	float3 LocalBoundsMax = 0;
	if (OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		LocalBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	}
	else if (FrustumGridUniformBuffer.bUseFrustumGrid)
	{
		LocalBoundsMax = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMax;
	}
	return LocalBoundsMax;
}

float3 GetLocalBoundsOrigin()
{
	return (GetLocalBoundsMin() + GetLocalBoundsMax()) / 2.0;
}

float3 GetLocalBoundsExtent()
{
	return (GetLocalBoundsMax() - GetLocalBoundsMin()) / 2.0;
}

float GetMaxTraceDistance()
{
	return 30000;
}

float GetMaxShadowTraceDistance()
{
	return 30000;
}

float GetStepSize()
{
	return 1.0;
}

float GetStepFactor()
{
	return 1.0;
}

float GetShadowStepSize()
{
	return ShadowStepSize;
}

float GetShadowStepFactor()
{
	return 1.0;
}

int GetStochasticFilteringMode()
{
	return StochasticFilteringMode;
}

float EvalAmbientOcclusion(float3 WorldPosition)
{
	return 1.0;
}

float GetIndirectInscatteringFactor()
{
	return 1.0;
}

uint3 GetVolumeResolution()
{
	return OrthoGridUniformBuffer.TopLevelGridResolution;
}

uint GetNumVoxels()
{
	return GetVolumeResolution().x * GetVolumeResolution().y * GetVolumeResolution().z;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float3 FilterWidth, float MipLevel, float Rand, int StochasticFilteringMode)
{
	float3 WorldPosition_DDX = mul(float4(LocalPosition + float3(1.0, 0.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	float3 WorldPosition_DDY = mul(float4(LocalPosition + float3(0.0, 1.0, 0.0), 1.0), GetLocalToWorld()).xyz;
	float3 WorldPosition_DDZ = mul(float4(LocalPosition + float3(0.0, 0.0, 1.0), 1.0), GetLocalToWorld()).xyz;

	if (StochasticFilteringMode == STOCHASTIC_FILTERING_CONSTANT)
	{
		LocalPosition = StochasticFilteringConstant(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRILINEAR)
	{
		LocalPosition = StochasticFilteringTrilinear(LocalPosition, Rand);
	}
	else if (StochasticFilteringMode == STOCHASTIC_FILTERING_TRICUBIC)
	{
		LocalPosition = StochasticFilteringTricubic(LocalPosition, Rand);
	}

	float3 WorldPosition2 = mul(float4(LocalPosition, 1.0), GetLocalToWorld()).xyz;

	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.WorldPosition = WorldPosition2;
	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(
	float3 LocalPosition,
	float3 WorldPosition,
	float3 WorldPosition_DDX,
	float3 WorldPosition_DDY,
	float MipLevel
)
{
	FVolumeSampleContext VolumeSampleContext;
	VolumeSampleContext.WorldPosition = WorldPosition;

	return VolumeSampleContext;
}

FVolumeSampleContext CreateVolumeSampleContext(float3 LocalPosition, float3 WorldPosition, float MipLevel)
{
	float3 WorldPosition_DDX = 0;
	float3 WorldPosition_DDY = 0;

	return CreateVolumeSampleContext(
		LocalPosition,
		WorldPosition,
		WorldPosition_DDX,
		WorldPosition_DDY,
		MipLevel
	);
}

float3 SampleExtinction(inout FVolumeSampleContext Context)
{
	return EvalExtinction(Context.WorldPosition);
}

float3 SampleEmission(inout FVolumeSampleContext Context)
{
	return EvalEmission(Context.WorldPosition);
}

float3 SampleAlbedo(inout FVolumeSampleContext Context)
{
	float3 Albedo = 0;

	float3 Extinction = EvalExtinction(Context.WorldPosition);
	if (any(Extinction > 0))
	{
		float3 Scattering = EvalAlbedo(Context.WorldPosition);
		Albedo = Scattering / Extinction;
	}

	return Albedo;
}

#define SUPPORT_CONTACT_SHADOWS 0

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

// Cinematic feature options
bool UseInscatteringVolume()
{
	return true;
}

#define INDIRECT_LIGHTING_MODE_OFF 0
#define INDIRECT_LIGHTING_MODE_LIGHTING_CACHE 1
#define INDIRECT_LIGHTING_MODE_SINGLE_SCATTERING 2

int GetIndirectLightingMode()
{
	return INDIRECT_LIGHTING_MODE_OFF;
}

bool IsOfflineRender()
{
	return false;
}

#define FOG_INSCATTERING_MODE_OFF 0
#define FOG_INSCATTERING_MODE_REFERENCE 1
#define FOG_INSCATTERING_MODE_LINEAR_APPROX 2

int GetFogInscatteringMode()
{
	return FOG_INSCATTERING_MODE_LINEAR_APPROX;
}

int ShouldApplyHeightFog()
{
	return false;
}

int ShouldApplyVolumetricFog()
{
	return false;
}

#include "HeterogeneousVolumesLightingUtils.ush"
#include "HeterogeneousVolumesRayMarchingUtils.ush"
#include "HeterogeneousVolumesAdaptiveVolumetricShadowMapUtils.ush"

int MaxSampleCount;
float AbsoluteErrorThreshold;
float RelativeErrorThreshold;

RWBuffer<int> RWVolumetricShadowLinkedListAllocatorBuffer;
RWStructuredBuffer<uint2> RWVolumetricShadowLinkedListBuffer;

uint GetLinearIndex(uint Face, uint2 PixelCoord)
{
	uint LinearIndex = Face * ShadowResolution.x * ShadowResolution.y + PixelCoord.y * ShadowResolution.x + PixelCoord.x;
	return LinearIndex;
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void ClearVolumetricShadowMapCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	// Create screen ray
	if (any(DispatchThreadId.xy >= ShadowResolution))
	{
		return;
	}

	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	uint LinearIndex = GetLinearIndex(Face, PixelCoord);
	int PixelIndex = LinearIndex * MaxSampleCount;

	float Depth = 0.0;
	float Transmittance = 1.0;
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_CreateSampleData(Depth, Transmittance));
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
}

// Debug
struct FVolumetricShadowMapDebugData
{
	float3 LightRayStart;
	float3 LightRayEnd;
	float3 RayOrigin;
	float3 RayEnd;
	float2 HitSpan;
};

RWStructuredBuffer<FVolumetricShadowMapDebugData> RWDebugBuffer;

uint FindSampleIndexWithLowestError(
	uint PixelIndex
)
{
	// Find sample, whose removal would mark the lowest overall geometric error
	uint LowestErrorIndex = 0;
	float LowestError = 1.0e6;
	for (uint Index = 1; Index < MaxSampleCount - 1; ++Index)
	{
		uint PrevIndex = AVSM_GetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]);
		uint NextIndex = AVSM_GetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]);

		FAVSMSampleData Data[3] = {
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevIndex]),
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + Index]),
			AVSM_GetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextIndex])
		};

		float Error = abs(
			(Data[2].X - Data[0].X) * (Data[2].Tau - Data[1].Tau) -
			(Data[2].X - Data[1].X) * (Data[2].Tau - Data[0].Tau)
		);

		if (Error < LowestError)
		{
			LowestError = Error;
			LowestErrorIndex = Index;
		}
	}

	return LowestErrorIndex;
}

void AddSample(FAVSMSampleData Data, int PixelIndex, inout int PrevSampleIndex, inout int SampleIndex)
{
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], Data);
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], PrevSampleIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], SampleIndex + 1);

	PrevSampleIndex = SampleIndex;
	SampleIndex = SampleIndex + 1;
}

void ReplaceSample(
	int PixelIndex,
	int SampleIndex,
	FAVSMSampleData SampleData,
	int PrevPtrIndex,
	int NextPtrIndex
)
{
	// Remove 
	{
		int PrevIndex = AVSM_GetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex]);
		int NextIndex = AVSM_GetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex]);

		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevIndex], NextIndex);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextIndex], PrevIndex);
	}

	// Replace
	AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], SampleData);
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], PrevPtrIndex);
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + SampleIndex], NextPtrIndex);

	// Update neighborhood
	AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevPtrIndex], SampleIndex);
	AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + NextPtrIndex], SampleIndex);
}

void AddOrReplaceSample(FAVSMSampleData Data, int PixelIndex, inout int PrevSampleIndex, inout int SampleIndex)
{
	// Add
	if (SampleIndex < MaxSampleCount - 1)
	{
		AddSample(Data, PixelIndex, PrevSampleIndex, SampleIndex);
	}
	// Or replace
	else
	{
#if USE_AVSM_COMPRESSION
		// Insert fictious sample at MaxSampleCount - 1 to compute error
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], Data);
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevSampleIndex], MaxSampleCount - 1);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], PrevSampleIndex);

		int LowestErrorIndex = FindSampleIndexWithLowestError(PixelIndex);
		if (LowestErrorIndex > 0)
		{
			if (LowestErrorIndex == PrevSampleIndex)
			{
				AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevSampleIndex], Data);
			}
			else
			{
				ReplaceSample(PixelIndex, LowestErrorIndex, Data, PrevSampleIndex, MaxSampleCount - 1);
			}

			PrevSampleIndex = LowestErrorIndex;
		}
#else
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], Data);
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + MaxSampleCount - 1], PrevSampleIndex);

		PrevSampleIndex = MaxSampleCount - 1;
#endif
	}
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void RenderVolumetricShadowMapForLightWithVoxelGridCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	// Create screen ray
	if (any(DispatchThreadId.xy >= ShadowResolution))
	{
		return;
	}
	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	uint LinearIndex = GetLinearIndex(Face, PixelCoord);

	// Generate a light ray
	//float2 ShadowUV = float2(PixelCoord + 0.5f) / float2(ShadowResolution);
	float2 ShadowUV = float2(PixelCoord + View.TemporalAAJitter.xy) / float2(ShadowResolution);
	const float NearScreenZ = 1.0;
	const float FarScreenZ = 0.0;

	float4 LightRayStartAsFloat4 = mul(float4(ShadowUV, NearScreenZ, 1), ShadowToTranslatedWorld[Face]);
	float4 LightRayEndAsFloat4 = mul(float4(ShadowUV, FarScreenZ, 1), ShadowToTranslatedWorld[Face]);
	float3 LightRayStart = LightRayStartAsFloat4.xyz * rcp(LightRayStartAsFloat4.w);
	float3 LightRayEnd = LightRayEndAsFloat4.xyz * rcp(LightRayEndAsFloat4.w);

	// Intersect ray with bounding volume
	// TODO: LWC integration..
	float3 WorldRayOrigin = DFFastSubtractDemote(LightRayStart, PrimaryView.PreViewTranslation);
	float3 WorldRayEnd = DFFastSubtractDemote(LightRayEnd, PrimaryView.PreViewTranslation);

	float3 WorldRayDirection = WorldRayEnd - WorldRayOrigin;
	float WorldRayLength = length(WorldRayDirection);
	WorldRayDirection /= WorldRayLength;

	float3 WorldBoundsMin = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMax;
	if (OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
		WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	}

	float2 RayHitT = IntersectAABB(
		WorldRayOrigin,
		WorldRayDirection,
		0.0,
		WorldRayLength,
		WorldBoundsMin,
		WorldBoundsMax
	);

	float LocalHitT = RayHitT.x;
	float WorldHitT = RayHitT.x;
	float3 Transmittance = 1;
	uint PixelIndex = LinearIndex * MaxSampleCount;

	float HitSpan = RayHitT.y - RayHitT.x;
	if (HitSpan > 0.0)
	{
		float Jitter = bJitter ? BlueNoiseScalar(PixelCoord, View.StateFrameIndexMod8) : 0.0;

		FRayMarchingContext RayMarchingContext = CreateRayMarchingContext(
			// Local-space
			WorldRayOrigin,
			WorldRayDirection,
			RayHitT.x,
			RayHitT.y,
			// World-space
			WorldRayOrigin,
			WorldRayDirection,
			// Ray-step attributes
			Jitter,
			CalcShadowStepSize(WorldRayDirection),
			MaxStepCount,
			//MaxSampleCount, // Replace max-steps with max-samples..
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance
		);

		RayMarchingContext.RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex);
		uint StepCount = CalcStepCount(RayMarchingContext);

		float3 ExpectedExtinction = 0.0;
		float3 ExpectedTransmittance = 1.0;
		uint PrevElementIndex = AVSM_NULL_PTR;
		uint ElementIndex = 0;

		for (uint StepIndex = 0; StepIndex < StepCount; ++StepIndex)
		{
			LocalHitT = min(RayMarchingContext.LocalRayTMin + RayMarchingContext.StepSize * (RayMarchingContext.Jitter + StepIndex), RayMarchingContext.LocalRayTMax);
			WorldHitT = LocalHitT * RayMarchingContext.LocalToWorldScale;
			float3 LocalPosition = RayMarchingContext.LocalRayOrigin + RayMarchingContext.LocalRayDirection * LocalHitT;
			float3 WorldPosition = RayMarchingContext.WorldRayOrigin + RayMarchingContext.WorldRayDirection * WorldHitT;

			FVolumeSampleContext SampleContext = CreateVolumeSampleContext(LocalPosition, WorldPosition, RayMarchingContext.MipLevel);
			float3 Extinction = SampleExtinction(SampleContext);
			Transmittance *= exp(-Extinction * RayMarchingContext.StepSize);

			bool bFirstEntry = (ElementIndex == 0);
			if (bFirstEntry)
			{
				FAVSMSampleData SampleData = AVSM_CreateSampleData(WorldHitT, Transmittance);
				AddSample(SampleData, PixelIndex, PrevElementIndex, ElementIndex);

				// Set expected values
				ExpectedExtinction = Extinction;
				ExpectedTransmittance = Transmittance;
			}
			else
			{
				// If extrapolated transmittance is beyond relative error tolerance
				ExpectedTransmittance *= exp(-ExpectedExtinction * RayMarchingContext.StepSize);
				float AbsoluteError = length(Transmittance - ExpectedTransmittance);
				float RelativeError = length((Transmittance - ExpectedTransmittance) / ExpectedTransmittance);
				//if ((AbsoluteError > AbsoluteErrorThreshold * (1 + RayMarchingContext.Jitter)) && (RelativeError > RelativeErrorThreshold * (1 + RayMarchingContext.Jitter)))
				if ((AbsoluteError > AbsoluteErrorThreshold) && (RelativeError > RelativeErrorThreshold))
				{
					FAVSMSampleData SampleData = AVSM_CreateSampleData(WorldHitT, Transmittance);
					AddOrReplaceSample(SampleData, PixelIndex, PrevElementIndex, ElementIndex);

					// Reset expected values
					ExpectedExtinction = Extinction;
					ExpectedTransmittance = Transmittance;
				}
			}

			// Early termination
			float Epsilon = 1.0e-7;
			if (all(Transmittance < Epsilon))
			{
				Transmittance = 0.0;
				break;
			}
		}

		// Stitch up the previous entry
		if (PrevElementIndex != AVSM_NULL_PTR)
		{
			AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevElementIndex], ElementIndex);

		}

		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], AVSM_CreateSampleData(WorldHitT, Transmittance.x));
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], PrevElementIndex);
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + ElementIndex], AVSM_NULL_PTR);
	}
	else
	{
		AVSM_SetLinkedListSampleData(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_CreateSampleData(0.0, Transmittance.x));
		AVSM_SetLinkedListPrevPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex], AVSM_NULL_PTR);
	}

	if (Face == ShadowDebugTweak)
	{
		RWBeerShadowMapTexture[PixelCoord] = float4(Transmittance, 1);
	}
}

StructuredBuffer<uint2> VolumetricShadowLinkedListBuffer;

RWBuffer<uint> RWVolumetricShadowIndirectionAllocatorBuffer;
RWStructuredBuffer<uint4> RWVolumetricShadowIndirectionBuffer;
RWStructuredBuffer<uint> RWVolumetricShadowTransmittanceBuffer;

groupshared int GSWarpSampleCount[THREADGROUP_SIZE_2D * THREADGROUP_SIZE_2D];
groupshared int GSWarpPrefixSum[THREADGROUP_SIZE_2D * THREADGROUP_SIZE_2D];
groupshared int GSWarpOffset;

/*
 * CompressVolumetricShadowMapCS
 * 
 * Builds a compressed Adaptive Volumetric Shadow Map. The structure utilizes two buffers:
 * 
 * 1) Per-pixel indirection buffer:
 *    a) Indirection pointer to transmittance function.
 *    b) Number of samples stored in the transmittance function (max = 64).
 *    c) Bounding depth information: (z_0, z_n)
 * 
 * 2) Transmittance function atlas:
 *    The atlas is a densely packed two-level structure, containing spans of transmittance
 *    information for a given pixel. To optimize for query performance, only the top-level
 *    contains the first element, offsetting the second-level elements by 1 to optimize
 *    for 4-wide loads.
 * 
 *    Level 1) Strided information to accelerate the query:
 *             (z_0, tau_0) -> (z_4, tau_4) -> (z_8, tau_8) -> (z_12, tau_12)
 * 
 *    Level 2) Linear span of transmittance information, offset by 1:
 *             (z_1, tau_1) -> (z_2, tau_2) -> ... -> (z_15, tau_15)
 */
[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void CompressVolumetricShadowMapCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint GroupIndex : SV_GroupIndex
)
{
	GSWarpSampleCount[GroupIndex] = 0;
	GSWarpOffset = AVSM_INVALID_PIXEL_OFFSET;

	// TODO: Early return, dependent on parallelizing pre-fix sum
#if 0
	if (any(DispatchThreadId >= ShadowResolution))
	{
		return;
	}
#endif
	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	uint LinearIndex = GetLinearIndex(Face, PixelCoord);
	int PixelIndex = LinearIndex * MaxSampleCount;

	// Calculate sample count
	int SampleCount = 0;
	FAVSMSampleData DataBegin = AVSM_CreateSampleData(0.0, 1.0);
	FAVSMSampleData DataEnd = DataBegin;
	if (all(PixelCoord < ShadowResolution))
	{
		DataBegin = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer[PixelIndex]);
		DataEnd = DataBegin;
		int NextPtr = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer[PixelIndex]);
		SampleCount = (NextPtr == AVSM_NULL_PTR) ? 0 : 1;
		while (NextPtr != AVSM_NULL_PTR)
		{
			SampleCount++;
			DataEnd = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer[PixelIndex + NextPtr]);
			NextPtr = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer[PixelIndex + NextPtr]);
		}
	}

	// Add top-level skip list
	int TopLevelSampleCount = CalcTopLevelSampleCount(SampleCount);
	GSWarpSampleCount[GroupIndex] = Align4(TopLevelSampleCount) + Align4(SampleCount);
	GroupMemoryBarrierWithGroupSync();

	// TODO: Parallel Prefix sum
	int PrefixSum = 0;
	for (int ThreadIndex = 0; ThreadIndex < GroupIndex; ++ThreadIndex)
	{
		PrefixSum += GSWarpSampleCount[ThreadIndex];
	}
	GSWarpPrefixSum[GroupIndex] = PrefixSum;
	GroupMemoryBarrierWithGroupSync();

	// Allocate for entire threadgroup
	uint LastIndex = THREADGROUP_SIZE_2D * THREADGROUP_SIZE_2D - 1;
	uint TotalSampleCount = GSWarpPrefixSum[LastIndex] + GSWarpSampleCount[LastIndex];
	if ((GroupIndex == 0) && (TotalSampleCount > 0))
	{
		InterlockedAdd(RWVolumetricShadowIndirectionAllocatorBuffer[0], TotalSampleCount, GSWarpOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	uint TopLevelOffset = GSWarpOffset + GSWarpPrefixSum[GroupIndex];
	uint BottomLevelOffset = GSWarpOffset + GSWarpPrefixSum[GroupIndex] + Align4(TopLevelSampleCount);

	// Write indirection entry
	FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
		GSWarpOffset + GSWarpPrefixSum[GroupIndex],
		SampleCount,
		DataBegin.X,
		DataEnd.X
	);

	if (all(PixelCoord < ShadowResolution))
	{
		RWVolumetricShadowIndirectionBuffer[LinearIndex] = AVSM_PackIndirectionData(IndirectionData);
	}

	// Pack and fill data per thread
	FAVSMSampleData ShadowData = AVSM_CreateSampleData(0.0, 1.0);
	int NextPtr = 0;
	for (int SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
	{
		ShadowData = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer[PixelIndex + NextPtr]);
		NextPtr = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer[PixelIndex + NextPtr]);

		// Store indexed list offset by 1 element
		int BottomLevelIndex = max(SampleIndex - 1, 0);
		RWVolumetricShadowTransmittanceBuffer[BottomLevelOffset + BottomLevelIndex] = AVSM_PackSampleData(ShadowData, IndirectionData);

		// Mark the current sample as an acceleration sample at the top-level
		if (SampleIndex % 4 == 0)
		{
			uint TopLevelIndex = SampleIndex / 4;
			RWVolumetricShadowTransmittanceBuffer[TopLevelOffset + TopLevelIndex] = AVSM_PackSampleData(ShadowData, IndirectionData);
		}
	}

	// Pad remainder
	for (int BottomLevelIndex = max(SampleCount - 1, 0); BottomLevelIndex < Align4(SampleCount); ++BottomLevelIndex)
	{
		RWVolumetricShadowTransmittanceBuffer[BottomLevelOffset + BottomLevelIndex] = AVSM_PackSampleData(AVSM_CreateSampleData(ShadowData.X, ShadowData.Tau), IndirectionData);
	}

	// Pad remainder
	for (int TopLevelIndex = TopLevelSampleCount; TopLevelIndex < Align4(TopLevelSampleCount); ++TopLevelIndex)
	{
		RWVolumetricShadowTransmittanceBuffer[TopLevelOffset + TopLevelIndex] = AVSM_PackSampleData(AVSM_CreateSampleData(ShadowData.X, ShadowData.Tau), IndirectionData);
	}
}

StructuredBuffer<uint2> VolumetricShadowLinkedListBuffer0;
StructuredBuffer<uint2> VolumetricShadowLinkedListBuffer1;

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void CombineVolumetricShadowMapsCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint GroupIndex : SV_GroupIndex
)
{
	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord >= ShadowResolution))
	{
		return;
	}

	uint LinearIndex = GetLinearIndex(Face, PixelCoord);
	int PixelIndex = LinearIndex * MaxSampleCount;

	int PrevSampleIndex = AVSM_NULL_PTR;
	int SampleIndex = 0;

	float Transmittance0 = 1.0;
	float Transmittance1 = 1.0;
	int IndexPtr0 = 0;
	int IndexPtr1 = 0;

	// Sorted sample insertion
	while ((IndexPtr0 != AVSM_NULL_PTR) && (IndexPtr1 != AVSM_NULL_PTR))
	{
		FAVSMSampleData Data0 = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr0]);
		FAVSMSampleData Data1 = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer1[PixelIndex + IndexPtr1]);

		FAVSMSampleData DataToInsert;
		if (Data0.X < Data1.X)
		{
			DataToInsert = Data0;
			DataToInsert.Tau *= Transmittance1;
			Transmittance0 = Data0.Tau;

			IndexPtr0 = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr0]);
		}
		else
		{
			DataToInsert = Data1;
			DataToInsert.Tau *= Transmittance0;
			Transmittance1 = Data1.Tau;

			IndexPtr1 = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer1[PixelIndex + IndexPtr1]);
		}

		AddOrReplaceSample(DataToInsert, PixelIndex, PrevSampleIndex, SampleIndex);
	}

	// Sample0 residual
	while (IndexPtr0 != AVSM_NULL_PTR)
	{
		FAVSMSampleData DataToInsert = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr0]);
		DataToInsert.Tau *= Transmittance1;

		IndexPtr0 = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr0]);
		AddOrReplaceSample(DataToInsert, PixelIndex, PrevSampleIndex, SampleIndex);
	}

	// Sample1 residual
	while ((IndexPtr1 != AVSM_NULL_PTR))
	{
		FAVSMSampleData DataToInsert = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer1[PixelIndex + IndexPtr1]);
		DataToInsert.Tau *= Transmittance0;

		IndexPtr1 = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer1[PixelIndex + IndexPtr1]);
		AddOrReplaceSample(DataToInsert, PixelIndex, PrevSampleIndex, SampleIndex);
	}

	// Mark the final sample as being terminal
	if (PrevSampleIndex != AVSM_NULL_PTR)
	{
		AVSM_SetLinkedListNextPtr(RWVolumetricShadowLinkedListBuffer[PixelIndex + PrevSampleIndex], AVSM_NULL_PTR);
	}
}

#define MAX_SAMPLE_COUNT 2

// Beer shadow map
//Texture2D<float4> BeerShadowMapTexture;
//int2 ShadowResolution;

struct FBeerShadowMapData
{
	float T[2];
	float Tau[2];
};

float4 PackBeerShadowMapData(FBeerShadowMapData Data)
{
	float4 PackedData;
	PackedData.r = Data.T[0];
	PackedData.g = Data.T[1];
	PackedData.b = Data.Tau[0];
	PackedData.a = Data.Tau[1];
	return PackedData;
}

FBeerShadowMapData UnpackBeerShadowMapData(float4 PackedData)
{
	FBeerShadowMapData Data;
	Data.T[0] = PackedData.r;
	Data.T[1] = PackedData.g;
	Data.Tau[0] = PackedData.b;
	Data.Tau[1] = PackedData.a;
	return Data;
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void ConvertBeerLawShadowMapToVolumetricShadowMapCS(
	uint2 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint ScalarGroupThreadId = GroupThreadId.y * THREADGROUP_SIZE_2D + GroupThreadId.x;
	if (ScalarGroupThreadId == 0)
	{
		GSWarpOffset = AVSM_INVALID_PIXEL_OFFSET;
	}

	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	uint LinearIndex = GetLinearIndex(Face, PixelCoord);

	// Allocation is fixed-size
	int BottomLevelSampleCount = MAX_SAMPLE_COUNT;
	int TopLevelSampleCount = CalcTopLevelSampleCount(BottomLevelSampleCount);
	int TotalSampleCount = Align4(TopLevelSampleCount) + Align4(BottomLevelSampleCount);

	uint TileSize = THREADGROUP_SIZE_2D * THREADGROUP_SIZE_2D;
	uint SampleCountPerTile = TotalSampleCount * TileSize;
	if ((ScalarGroupThreadId == 0) && (SampleCountPerTile > 0))
	{
		InterlockedAdd(RWVolumetricShadowIndirectionAllocatorBuffer[0], SampleCountPerTile, GSWarpOffset);
	}
	GroupMemoryBarrierWithGroupSync();

	if (any(PixelCoord < 0) || any(PixelCoord >= ShadowResolution))
	{
		return;
	}

	uint TopLevelOffset = GSWarpOffset + ScalarGroupThreadId * TotalSampleCount;
	uint BottomLevelOffset = TopLevelOffset + Align4(TopLevelSampleCount);

	FBeerShadowMapData BeerShadowMapData = UnpackBeerShadowMapData(BeerShadowMapTexture[PixelCoord]);

	// Write indirection entry
	int PixelOffset = TopLevelOffset;
	FAVSMIndirectionData IndirectionData = AVSM_CreateIndirectionData(
		PixelOffset,
		BottomLevelSampleCount,
		BeerShadowMapData.T[0],
		BeerShadowMapData.T[1]
	);

	RWVolumetricShadowIndirectionBuffer[LinearIndex] = AVSM_PackIndirectionData(IndirectionData);

	// Convert beer shadow map samples to AVSM samples
	FAVSMSampleData ShadowData[] = {
		AVSM_CreateSampleData(BeerShadowMapData.T[0], BeerShadowMapData.Tau[0]),
		AVSM_CreateSampleData(BeerShadowMapData.T[1], BeerShadowMapData.Tau[1])
	};

	for (int SampleIndex = 0; SampleIndex < BottomLevelSampleCount; ++SampleIndex)
	{
		// Store indexed list offset by 1 element
		int BottomLevelIndex = max(SampleIndex - 1, 0);
		RWVolumetricShadowTransmittanceBuffer[BottomLevelOffset + BottomLevelIndex] = AVSM_PackSampleData(ShadowData[SampleIndex], IndirectionData);

		// Mark the current sample as an acceleration sample at the top-level
		if (SampleIndex % 4 == 0)
		{
			uint TopLevelIndex = SampleIndex / 4;
			RWVolumetricShadowTransmittanceBuffer[TopLevelOffset + TopLevelIndex] = AVSM_PackSampleData(ShadowData[SampleIndex], IndirectionData);
		}
	}
	
	// Pad remainder
	for (int BottomLevelIndex = max(BottomLevelSampleCount - 1, 0); BottomLevelIndex < Align4(BottomLevelSampleCount); ++BottomLevelIndex)
	{
		RWVolumetricShadowTransmittanceBuffer[BottomLevelOffset + BottomLevelIndex] = AVSM_PackSampleData(AVSM_CreateSampleData(ShadowData[1].X, ShadowData[1].Tau), IndirectionData);
	}

	// Pad remainder
	for (int TopLevelIndex = TopLevelSampleCount; TopLevelIndex < Align4(TopLevelSampleCount); ++TopLevelIndex)
	{
		RWVolumetricShadowTransmittanceBuffer[TopLevelOffset + TopLevelIndex] = AVSM_PackSampleData(AVSM_CreateSampleData(ShadowData[1].X, ShadowData[1].Tau), IndirectionData);
	}
}

[numthreads(THREADGROUP_SIZE_2D, THREADGROUP_SIZE_2D, 1)]
void ConvertVolumetricShadowMapToBeerLawShadowMapCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint Face = DispatchThreadId.z;
	uint2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord >= ShadowResolution))
	{
		return;
	}

	uint LinearIndex = GetLinearIndex(Face, PixelCoord);
	int PixelIndex = LinearIndex * MaxSampleCount;

	FAVSMSampleData SampleData[] = {
		AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer0[PixelIndex]),
		AVSM_CreateSampleData(0.0, 1.0)
	};

	int IndexPtr = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer0[PixelIndex]);
	while (IndexPtr != AVSM_NULL_PTR)
	{
		SampleData[1] = AVSM_GetLinkedListSampleData(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr]);
		IndexPtr = AVSM_GetLinkedListNextPtr(VolumetricShadowLinkedListBuffer0[PixelIndex + IndexPtr]);
	}

	FBeerShadowMapData BeerData;
	BeerData.T[0] = SampleData[0].X;
	BeerData.T[1] = SampleData[1].X;
	BeerData.Tau[0] = SampleData[0].Tau;
	BeerData.Tau[1] = SampleData[1].Tau;
	uint2 BeerPixelCoord = uint2(Face * ShadowResolution.x + PixelCoord.x, PixelCoord.y);
	RWBeerShadowMapTexture[BeerPixelCoord] = PackBeerShadowMapData(BeerData);
}
