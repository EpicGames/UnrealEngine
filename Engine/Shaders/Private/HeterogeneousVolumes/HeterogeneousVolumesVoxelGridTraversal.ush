// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeterogeneousVolumesVoxelGridUtils.ush"
#include "HeterogeneousVolumesFrustumVoxelGridUtils.ush"

void MarchBottomLevelVoxelGrid(
	float3 VoxelRayOrigin,
	float3 VoxelRayDirection,
	float VoxelRayTMin,
	float VoxelRayTMax,
	float WorldRayHitSpan,
	float3 TopLevelVoxelStartPos,
	float3 TopLevelVoxelEndPos,
	FTopLevelGridData TopLevelGridData,
	StructuredBuffer<FScalarGridData> ExtinctionGridBuffer,
	inout float3 Transmittance
)
{
	uint BottomLevelVoxelIndex = GetBottomLevelIndex(TopLevelGridData);
	int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);
	float TopLevelVoxelRayLength = length(TopLevelVoxelEndPos - TopLevelVoxelStartPos);

	// Transform to local-space
	float3 MinTopLevelVoxelPos = floor((TopLevelVoxelStartPos + TopLevelVoxelEndPos) * 0.5);
	float3 LocalRayOrigin = (TopLevelVoxelStartPos - MinTopLevelVoxelPos) * BottomLevelVoxelResolution;
	float3 LocalRayEnd = (TopLevelVoxelEndPos - MinTopLevelVoxelPos) * BottomLevelVoxelResolution;

	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;
	float3 LocalRayDirectionInv = 1.0 / LocalRayDirection;

	float LocalRayMarchT = 0.0;
	float3 LocalRayMarchPos = LocalRayOrigin;

	float3 LocalBoundsPos;
	LocalBoundsPos.x = sign(LocalRayDirection.x) > 0 ? floor(LocalRayMarchPos.x) + 1 : ceil(LocalRayMarchPos.x) - 1;
	LocalBoundsPos.y = sign(LocalRayDirection.y) > 0 ? floor(LocalRayMarchPos.y) + 1 : ceil(LocalRayMarchPos.y) - 1;
	LocalBoundsPos.z = sign(LocalRayDirection.z) > 0 ? floor(LocalRayMarchPos.z) + 1 : ceil(LocalRayMarchPos.z) - 1;

	float3 LocalBoundsHitT = (LocalBoundsPos - LocalRayMarchPos) * LocalRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (LocalBoundsHitT.x <= Epsilon) LocalBoundsHitT.x += abs(LocalRayDirectionInv.x);
	if (LocalBoundsHitT.y <= Epsilon) LocalBoundsHitT.y += abs(LocalRayDirectionInv.y);
	if (LocalBoundsHitT.z <= Epsilon) LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);

	float3 OpacityThreshold = 1.0e-4;
	while ((LocalRayMarchT < LocalRayLength) && all(Transmittance > OpacityThreshold))
	{
		float LocalRayMarchDeltaT = 0.0;
		if (LocalBoundsHitT.x < LocalBoundsHitT.y)
		{
			if (LocalBoundsHitT.x < LocalBoundsHitT.z)
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.x - LocalRayMarchT;
				LocalBoundsHitT.x += abs(LocalRayDirectionInv.x);
			}
			else
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.z - LocalRayMarchT;
				LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);
			}
		}
		else
		{
			if (LocalBoundsHitT.y < LocalBoundsHitT.z)
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.y - LocalRayMarchT;
				LocalBoundsHitT.y += abs(LocalRayDirectionInv.y);
			}
			else
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.z - LocalRayMarchT;
				LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (LocalRayMarchT + LocalRayMarchDeltaT > LocalRayLength)
		{
			LocalRayMarchDeltaT = LocalRayLength - LocalRayMarchT;
		}
		float3 LocalEndPos = LocalRayMarchPos + LocalRayDirection * LocalRayMarchDeltaT;
		float3 LocalSamplePos = clamp((LocalRayMarchPos + LocalEndPos) * 0.5, 0.0, BottomLevelVoxelResolution - 1.0);
		uint LocalSampleLinearPos = BottomLevelVoxelIndex + MortonEncode3(LocalSamplePos);

		float3 Extinction = GetExtinction(ExtinctionGridBuffer[LocalSampleLinearPos]);
		float WorldRayMarchDeltaT = max((LocalRayMarchDeltaT / LocalRayLength) * TopLevelVoxelRayLength * WorldRayHitSpan / VoxelRayTMax, 0.0);
		Transmittance *= saturate(exp(-Extinction * WorldRayMarchDeltaT));

		LocalRayMarchT += LocalRayMarchDeltaT;
		LocalRayMarchPos = LocalEndPos;
	}
}

void MarchIndirectionOrthoVoxelGrid(
	float3 VoxelRayOrigin,
	float3 VoxelRayDirection,
	float VoxelRayTMin,
	float VoxelRayTMax,
	float WorldRayHitSpan,
	float3 TopLevelVoxelStartPos,
	float3 TopLevelVoxelEndPos,
	FTopLevelGridData TopLevelGridData,
	StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer,
	StructuredBuffer<FScalarGridData> ExtinctionGridBuffer,
	inout float3 Transmittance
)
{
	uint BottomLevelVoxelIndex = GetBottomLevelIndex(TopLevelGridData);
	int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);
	float TopLevelVoxelRayLength = length(TopLevelVoxelEndPos - TopLevelVoxelStartPos);

	// Transform to local-space
	float3 MinTopLevelVoxelPos = floor((TopLevelVoxelStartPos + TopLevelVoxelEndPos) * 0.5);
	float3 LocalRayOrigin = (TopLevelVoxelStartPos - MinTopLevelVoxelPos) * BottomLevelVoxelResolution;
	float3 LocalRayEnd = (TopLevelVoxelEndPos - MinTopLevelVoxelPos) * BottomLevelVoxelResolution;

	float3 LocalRayDirection = LocalRayEnd - LocalRayOrigin;
	float LocalRayLength = length(LocalRayDirection);
	LocalRayDirection /= LocalRayLength;
	float3 LocalRayDirectionInv = 1.0 / LocalRayDirection;

	float LocalRayMarchT = 0.0;
	float3 LocalRayMarchPos = LocalRayOrigin;

	float3 LocalBoundsPos;
	LocalBoundsPos.x = sign(LocalRayDirection.x) > 0 ? floor(LocalRayMarchPos.x) + 1 : ceil(LocalRayMarchPos.x) - 1;
	LocalBoundsPos.y = sign(LocalRayDirection.y) > 0 ? floor(LocalRayMarchPos.y) + 1 : ceil(LocalRayMarchPos.y) - 1;
	LocalBoundsPos.z = sign(LocalRayDirection.z) > 0 ? floor(LocalRayMarchPos.z) + 1 : ceil(LocalRayMarchPos.z) - 1;

	float3 LocalBoundsHitT = (LocalBoundsPos - LocalRayMarchPos) * LocalRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (LocalBoundsHitT.x <= Epsilon) LocalBoundsHitT.x += abs(LocalRayDirectionInv.x);
	if (LocalBoundsHitT.y <= Epsilon) LocalBoundsHitT.y += abs(LocalRayDirectionInv.y);
	if (LocalBoundsHitT.z <= Epsilon) LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);

	float3 OpacityThreshold = 1.0e-4;
	while ((LocalRayMarchT < LocalRayLength) && all(Transmittance > OpacityThreshold))
	{
		float LocalRayMarchDeltaT = 0.0;
		if (LocalBoundsHitT.x < LocalBoundsHitT.y)
		{
			if (LocalBoundsHitT.x < LocalBoundsHitT.z)
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.x - LocalRayMarchT;
				LocalBoundsHitT.x += abs(LocalRayDirectionInv.x);
			}
			else
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.z - LocalRayMarchT;
				LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);
			}
		}
		else
		{
			if (LocalBoundsHitT.y < LocalBoundsHitT.z)
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.y - LocalRayMarchT;
				LocalBoundsHitT.y += abs(LocalRayDirectionInv.y);
			}
			else
			{
				LocalRayMarchDeltaT = LocalBoundsHitT.z - LocalRayMarchT;
				LocalBoundsHitT.z += abs(LocalRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (LocalRayMarchT + LocalRayMarchDeltaT > LocalRayLength)
		{
			LocalRayMarchDeltaT = LocalRayLength - LocalRayMarchT;
		}

		float3 LocalEndPos = LocalRayMarchPos + LocalRayDirection * LocalRayMarchDeltaT;
		float3 LocalSamplePos = clamp((LocalRayMarchPos + LocalEndPos) * 0.5, 0.0, BottomLevelVoxelResolution - 1.0);
		uint LocalSampleLinearPos = BottomLevelVoxelIndex + MortonEncode3(LocalSamplePos);
		float LocalScaleFactor = TopLevelVoxelRayLength / LocalRayLength;

		FTopLevelGridData IndirectionData = IndirectionGridBuffer[LocalSampleLinearPos];
		if (IsBottomLevelAllocated(IndirectionData))
		{
			MarchBottomLevelVoxelGrid(
				LocalRayOrigin,
				LocalRayDirection,
				VoxelRayTMin,
				VoxelRayTMax,
				WorldRayHitSpan * LocalScaleFactor,
				LocalRayMarchPos,
				LocalEndPos,
				IndirectionData,
				ExtinctionGridBuffer,
				Transmittance
			);
		}

		LocalRayMarchT += LocalRayMarchDeltaT;
		LocalRayMarchPos = LocalEndPos;
	}
}

void MarchTopLevelOrthoVoxelGrid(
	float3 VoxelRayOrigin,
	float3 VoxelRayDirection,
	float VoxelRayTMin,
	float VoxelRayTMax,
	float WorldRayHitSpan,
	bool bEnableIndirectionGrid,
	int3 TopLevelGridResolution,
	StructuredBuffer<FTopLevelGridData> TopLevelGridBuffer,
	StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer,
	StructuredBuffer<FScalarGridData> ExtinctionGridBuffer,
	inout float3 Transmittance
)
{
	float3 VoxelRayDirectionInv = 1.0 / VoxelRayDirection;

	// Grid intersection
	float VoxelRayMarchT = VoxelRayTMin;
	float3 VoxelRayMarchPos = VoxelRayOrigin;

	float3 VoxelBoundsPos;
	VoxelBoundsPos.x = sign(VoxelRayDirection.x) > 0 ? floor(VoxelRayMarchPos.x) + 1 : ceil(VoxelRayMarchPos.x) - 1;
	VoxelBoundsPos.y = sign(VoxelRayDirection.y) > 0 ? floor(VoxelRayMarchPos.y) + 1 : ceil(VoxelRayMarchPos.y) - 1;
	VoxelBoundsPos.z = sign(VoxelRayDirection.z) > 0 ? floor(VoxelRayMarchPos.z) + 1 : ceil(VoxelRayMarchPos.z) - 1;

	float3 VoxelBoundsHitT = (VoxelBoundsPos - VoxelRayMarchPos) * VoxelRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (VoxelBoundsHitT.x <= Epsilon) VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
	if (VoxelBoundsHitT.y <= Epsilon) VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
	if (VoxelBoundsHitT.z <= Epsilon) VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);

	float3 OpacityThreshold = 1.0e-4;
	while ((VoxelRayMarchT < VoxelRayTMax) && all(Transmittance > OpacityThreshold))
	{
		float VoxelRayMarchDeltaT = 0.0;
		if (VoxelBoundsHitT.x < VoxelBoundsHitT.y)
		{
			if (VoxelBoundsHitT.x < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.x - VoxelRayMarchT;
				VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}
		else
		{
			if (VoxelBoundsHitT.y < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.y - VoxelRayMarchT;
				VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (VoxelRayMarchT + VoxelRayMarchDeltaT > VoxelRayTMax)
		{
			VoxelRayMarchDeltaT = VoxelRayTMax - VoxelRayMarchT;
		}

		// Sample at the midpoint of the voxel ray
		float3 VoxelEndPos = VoxelRayMarchPos + VoxelRayDirection * VoxelRayMarchDeltaT;
		float3 VoxelSamplePos = (VoxelRayMarchPos + VoxelEndPos) * 0.5;
		uint VoxelSampleLinearPos = GetLinearIndex(VoxelSamplePos, TopLevelGridResolution);

		int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridBuffer[VoxelSampleLinearPos]);
		if (IsBottomLevelAllocated(TopLevelGridBuffer[VoxelSampleLinearPos]))
		{
			if (bEnableIndirectionGrid)
			{
				MarchIndirectionOrthoVoxelGrid(
					VoxelRayOrigin,
					VoxelRayDirection,
					VoxelRayTMin,
					VoxelRayTMax,
					WorldRayHitSpan,
					VoxelRayMarchPos,
					VoxelEndPos,
					TopLevelGridBuffer[VoxelSampleLinearPos],
					IndirectionGridBuffer,
					ExtinctionGridBuffer,
					Transmittance
				);
			}
			else if (all(BottomLevelVoxelResolution > 1))
			{
				MarchBottomLevelVoxelGrid(
					VoxelRayOrigin,
					VoxelRayDirection,
					VoxelRayTMin,
					VoxelRayTMax,
					WorldRayHitSpan,
					VoxelRayMarchPos,
					VoxelEndPos,
					TopLevelGridBuffer[VoxelSampleLinearPos],
					ExtinctionGridBuffer,
					Transmittance
				);
			}
			else
			{
				float3 Extinction = GetExtinction(TopLevelGridBuffer[VoxelSampleLinearPos], ExtinctionGridBuffer);
				float WorldRayMarchDeltaT = max(VoxelRayMarchDeltaT * WorldRayHitSpan / VoxelRayTMax, 0.0);
				Transmittance *= saturate(exp(-Extinction * WorldRayMarchDeltaT));
			}
		}

		VoxelRayMarchT += VoxelRayMarchDeltaT;
		VoxelRayMarchPos = VoxelEndPos;
	}
}

float3 TraceOrthoVoxelGrid(float3 Origin, float3 Direction, float TMin, float TMax)
{
	float3 Transmittance = 1.0;

	// Intersect world-space AABB to clip ray
	float3 WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;

	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(Origin, PrimaryView.PreViewTranslation));
	float2 ClipHitT = IntersectAABB(WorldRayOrigin, Direction, TMin, TMax, WorldBoundsMin, WorldBoundsMax);
	if (ClipHitT.x > ClipHitT.y)
	{
		return Transmittance;
	}

	float3 WorldRayBegin = WorldRayOrigin + Direction * ClipHitT.x;
	float3 WorldRayEnd = WorldRayOrigin + Direction * ClipHitT.y;
	float WorldRayLength = ClipHitT.y - ClipHitT.x;

	// Transform to voxel-space
	float3 WorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 VoxelRayBegin = (WorldRayBegin - WorldBoundsMin) / WorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayEnd = (WorldRayEnd - WorldBoundsMin) / WorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayDirection = VoxelRayEnd - VoxelRayBegin;
	float VoxelRayTMin = 0.0;
	float VoxelRayTMax = length(VoxelRayDirection);
	VoxelRayDirection /= VoxelRayTMax;

	MarchTopLevelOrthoVoxelGrid(
		VoxelRayBegin,
		VoxelRayDirection,
		VoxelRayTMin,
		VoxelRayTMax,
		WorldRayLength,
		OrthoGridUniformBuffer.bEnableIndirectionGrid,
		OrthoGridUniformBuffer.TopLevelGridResolution,
		OrthoGridUniformBuffer.TopLevelGridBuffer,
		OrthoGridUniformBuffer.IndirectionGridBuffer,
		OrthoGridUniformBuffer.ExtinctionGridBuffer,
		Transmittance
	);

	return Transmittance;
}

void MarchTopLevelFrustumVoxelGrid(
	float3 VoxelRayOrigin,
	float3 VoxelRayDirection,
	float VoxelRayTMin,
	float VoxelRayTMax,
	float WorldRayHitSpan,
	int3 TopLevelGridResolution,
	StructuredBuffer<FTopLevelGridData> TopLevelGridBuffer,
	StructuredBuffer<FScalarGridData> ExtinctionGridBuffer,
	inout float3 Transmittance
)
{
	float3 VoxelRayDirectionInv = 1.0 / VoxelRayDirection;

	// Grid intersection
	float VoxelRayMarchT = VoxelRayTMin;
	float3 VoxelRayMarchPos = VoxelRayOrigin;

	float3 VoxelBoundsPos;
	VoxelBoundsPos.x = sign(VoxelRayDirection.x) > 0 ? floor(VoxelRayMarchPos.x) + 1 : ceil(VoxelRayMarchPos.x) - 1;
	VoxelBoundsPos.y = sign(VoxelRayDirection.y) > 0 ? floor(VoxelRayMarchPos.y) + 1 : ceil(VoxelRayMarchPos.y) - 1;
	VoxelBoundsPos.z = sign(VoxelRayDirection.z) > 0 ? floor(VoxelRayMarchPos.z) + 1 : ceil(VoxelRayMarchPos.z) - 1;

	float3 VoxelBoundsHitT = (VoxelBoundsPos - VoxelRayMarchPos) * VoxelRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (VoxelBoundsHitT.x <= Epsilon) VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
	if (VoxelBoundsHitT.y <= Epsilon) VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
	if (VoxelBoundsHitT.z <= Epsilon) VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);

	float3 OpacityThreshold = 1.0e-4;
	while ((VoxelRayMarchT < VoxelRayTMax) && all(Transmittance > OpacityThreshold))
	{
		float VoxelRayMarchDeltaT = 0.0;
		if (VoxelBoundsHitT.x < VoxelBoundsHitT.y)
		{
			if (VoxelBoundsHitT.x < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.x - VoxelRayMarchT;
				VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}
		else
		{
			if (VoxelBoundsHitT.y < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.y - VoxelRayMarchT;
				VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (VoxelRayMarchT + VoxelRayMarchDeltaT > VoxelRayTMax)
		{
			VoxelRayMarchDeltaT = VoxelRayTMax - VoxelRayMarchT;
		}

		// Sample at the midpoint of the voxel ray
		float3 VoxelEndPos = VoxelRayMarchPos + VoxelRayDirection * VoxelRayMarchDeltaT;
		float3 VoxelSamplePos = (VoxelRayMarchPos + VoxelEndPos) * 0.5;
		uint VoxelSampleLinearPos = GetLinearIndex(VoxelSamplePos, TopLevelGridResolution);

		int3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridBuffer[VoxelSampleLinearPos]);
		if (IsBottomLevelAllocated(TopLevelGridBuffer[VoxelSampleLinearPos]))
		{
			if (all(BottomLevelVoxelResolution > 1))
			{
				MarchBottomLevelVoxelGrid(
					VoxelRayOrigin,
					VoxelRayDirection,
					VoxelRayTMin,
					VoxelRayTMax,
					WorldRayHitSpan,
					VoxelRayMarchPos,
					VoxelEndPos,
					TopLevelGridBuffer[VoxelSampleLinearPos],
					ExtinctionGridBuffer,
					Transmittance
				);
			}
			else
			{
				float3 Extinction = GetExtinction(TopLevelGridBuffer[VoxelSampleLinearPos], ExtinctionGridBuffer);
				float WorldRayMarchDeltaT = max(VoxelRayMarchDeltaT * WorldRayHitSpan / VoxelRayTMax, 0.0);
				Transmittance *= saturate(exp(-Extinction * WorldRayMarchDeltaT));
			}
		}

		VoxelRayMarchT += VoxelRayMarchDeltaT;
		VoxelRayMarchPos = VoxelEndPos;
	}
}

float3 TraceFrustumVoxelGrid(float3 Origin, float3 Direction, inout float TMin, inout float TMax)
{
	float3 Transmittance = 1.0;

	float3 WorldRayOrigin = DFHackToFloat(DFFastSubtract(Origin, PrimaryView.PreViewTranslation));
	float3 ViewRayOrigin = mul(float4(WorldRayOrigin, 1), FrustumGridUniformBuffer.WorldToView).xyz;
	float3 ViewRayDirection = mul(float4(WorldRayOrigin + Direction, 1), FrustumGridUniformBuffer.WorldToView).xyz;
	ViewRayDirection -= ViewRayOrigin;

	int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
	float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
	float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
	float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
	float3 VoxelRayOrigin = ViewToVoxel(ViewRayOrigin, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);
	float3 VoxelRayDirection = ViewToVoxel(ViewRayOrigin + ViewRayDirection, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);
	VoxelRayDirection -= VoxelRayOrigin;
	float UnitWorldToVoxelScale = length(VoxelRayDirection);
	VoxelRayDirection /= UnitWorldToVoxelScale;

	// Intersect voxel-space AABB to clip ray
	float VoxelRayTMin = TMin * UnitWorldToVoxelScale;
	float VoxelRayTMax = (TMax == POSITIVE_INFINITY) ? POSITIVE_INFINITY : TMax * UnitWorldToVoxelScale;
	float2 ClipHitT = IntersectAABB(VoxelRayOrigin, VoxelRayDirection, VoxelRayTMin, VoxelRayTMax, 0, VoxelDimensions);
	if (ClipHitT.x > ClipHitT.y)
	{
		TMin = TMax = 0;
		return Transmittance;
	}

	VoxelRayOrigin += VoxelRayDirection * ClipHitT.x;
	VoxelRayTMin = 0.0;
	VoxelRayTMax = ClipHitT.y - ClipHitT.x;

	float UnitVoxelToWorldScale = 1.0 / UnitWorldToVoxelScale;
	float WorldRayLength = VoxelRayTMax * UnitVoxelToWorldScale;

	MarchTopLevelFrustumVoxelGrid(
		VoxelRayOrigin,
		VoxelRayDirection,
		VoxelRayTMin,
		VoxelRayTMax,
		WorldRayLength,
		FrustumGridUniformBuffer.TopLevelFroxelGridResolution,
		FrustumGridUniformBuffer.TopLevelFroxelGridBuffer,
		FrustumGridUniformBuffer.ExtinctionFroxelGridBuffer,
		Transmittance
	);

	TMin = ClipHitT.x * UnitVoxelToWorldScale;
	TMax = ClipHitT.y * UnitVoxelToWorldScale;
	return Transmittance;
}
