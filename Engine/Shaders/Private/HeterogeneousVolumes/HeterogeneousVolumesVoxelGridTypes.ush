// Copyright Epic Games, Inc. All Rights Reserved.

struct FScalarGridData
{
	uint PackedData[2];
};

struct FVectorGridData
{
	uint PackedData[2];
};

struct FTopLevelGridData
{
	uint PackedData[1];
};

struct FTopLevelGridBitmaskData
{
	uint PackedData[2];
};
