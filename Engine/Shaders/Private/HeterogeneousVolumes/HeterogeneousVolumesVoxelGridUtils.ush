// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../MortonCode.ush"

#define EMPTY_VOXEL_INDEX 0x1FFFFFFF

struct FMajorantData
{
	float Majorant;
};

FMajorantData CreateMajorantData()
{
	FMajorantData MajorantData;
	MajorantData.Majorant = 0.0;
	return MajorantData;
}

void UpdateMajorantData(inout FMajorantData MajorantData, float3 Value)
{
	float MaxValue = max(Value.x, max(Value.y, Value.z));
	MajorantData.Majorant = max(MajorantData.Majorant, MaxValue);
}

FMajorantData CreateMajorantData(float3 Majorant)
{
	FMajorantData MajorantData = CreateMajorantData();
	UpdateMajorantData(MajorantData, Majorant);
	return MajorantData;
}

// Updating majorant data should occur for a single media
void UpdateMajorantData(inout FMajorantData MajorantData, FMajorantData Value)
{
	MajorantData.Majorant = max(MajorantData.Majorant, Value.Majorant);
}

// Merging majorant data should occur when mixing media
void MergeMajorantData(inout FMajorantData MajorantData, FMajorantData OverlappingMajorant)
{
	MajorantData.Majorant += OverlappingMajorant.Majorant;
}

FMajorantData GetMajorantData(FScalarGridData MajorantGridData)
{
	FMajorantData Data;
	Data.Majorant = asfloat(MajorantGridData.PackedData[1]);
	return Data;
}

void SetMajorantData(inout FScalarGridData MajorantGridData, FMajorantData MajorantData)
{
	MajorantGridData.PackedData[1] = asuint(MajorantData.Majorant);
}

float3 GetExtinction(FScalarGridData ExtinctionGridData)
{
	float3 Extinction = 0.0;
	Extinction.x = f16tof32(ExtinctionGridData.PackedData[0]);
	Extinction.y = f16tof32(ExtinctionGridData.PackedData[0] >> 16);
	Extinction.z = f16tof32(ExtinctionGridData.PackedData[1]);
	return Extinction;
}

void SetExtinction(inout FScalarGridData ExtinctionGridData, float3 Extinction)
{
	ExtinctionGridData.PackedData[0] = f32tof16(Extinction.x) | (f32tof16(Extinction.y) << 16);
	ExtinctionGridData.PackedData[1] = f32tof16(Extinction.z) | (ExtinctionGridData.PackedData[1] & 0xFFFF0000);
}

float3 GetEmission(FVectorGridData EmissionGridData)
{
	float3 Emission = 0.0;
	Emission.x = f16tof32(EmissionGridData.PackedData[0]);
	Emission.y = f16tof32(EmissionGridData.PackedData[0] >> 16);
	Emission.z = f16tof32(EmissionGridData.PackedData[1]);
	return Emission;
}

void SetEmission(inout FVectorGridData EmissionGridData, float3 Emission)
{
	EmissionGridData.PackedData[0] = f32tof16(Emission.x) | (f32tof16(Emission.y) << 16);
	EmissionGridData.PackedData[1] = f32tof16(Emission.z) | (EmissionGridData.PackedData[1] & 0xFFFF0000);
}

float3 GetAlbedo(FVectorGridData AlbedoGridData)
{
	float3 Albedo = 0.0;
	Albedo.x = f16tof32(AlbedoGridData.PackedData[0]);
	Albedo.y = f16tof32(AlbedoGridData.PackedData[0] >> 16);
	Albedo.z = f16tof32(AlbedoGridData.PackedData[1]);
	return Albedo;
}

void SetAlbedo(inout FVectorGridData AlbedoGridData, float3 Albedo)
{
	AlbedoGridData.PackedData[0] = f32tof16(Albedo.x) | (f32tof16(Albedo.y) << 16);
	AlbedoGridData.PackedData[1] = f32tof16(Albedo.z) | (AlbedoGridData.PackedData[1] & 0xFFFF0000);
}

FTopLevelGridData CreateTopLevelGridData(uint PackedData)
{
	FTopLevelGridData TopLevelGridData;
	TopLevelGridData.PackedData[0] = PackedData;
	return TopLevelGridData;
}

uint MortonEncode3(uint3 Voxel)
{
	return MortonCode3(Voxel.x) | MortonCode3(Voxel.y) << 1 | MortonCode3(Voxel.z) << 2;
}

uint3 MortonDecode3(uint Morton)
{
	uint3 Voxel = uint3(ReverseMortonCode3(Morton), ReverseMortonCode3(Morton >> 1), ReverseMortonCode3(Morton >> 2));
	return Voxel;
}

int GetLinearIndex(uint3 VoxelIndex, int3 TopLevelGridResolution)
{
	int SliceSize = TopLevelGridResolution.x * TopLevelGridResolution.y;

	return 
		VoxelIndex.z * SliceSize +
		VoxelIndex.y * TopLevelGridResolution.x +
		VoxelIndex.x;
}

int3 GetVoxelIndex(int LinearIndex, int3 TopLevelGridResolution)
{
	int SliceSize = TopLevelGridResolution.x * TopLevelGridResolution.y;
	int SliceIndex = LinearIndex / SliceSize;
	int SliceRemainder = LinearIndex - SliceIndex * SliceSize;
	
	return int3(
		SliceRemainder % TopLevelGridResolution.x,
		SliceRemainder / TopLevelGridResolution.x,
		SliceIndex
	);
}

uint GetBottomLevelIndex(FTopLevelGridData TopLevelGridData)
{
	// Maximum addressable space is equivalent to 1024 x 1024 x 512 top-level volume
	return TopLevelGridData.PackedData[0] >> 3;
}

void SetBottomLevelIndex(inout FTopLevelGridData TopLevelGridData, uint BottomLevelIndex)
{
	uint Index = BottomLevelIndex;
	uint ResolutionExponent = TopLevelGridData.PackedData[0] & 0x7;
	TopLevelGridData.PackedData[0] = (Index << 3) | ResolutionExponent;
}

bool IsBottomLevelAllocated(FTopLevelGridData TopLevelGridData)
{
	return GetBottomLevelIndex(TopLevelGridData) != EMPTY_VOXEL_INDEX;
}

bool IsBottomLevelEmpty(FTopLevelGridData TopLevelGridData)
{
	uint ResolutionExponent = TopLevelGridData.PackedData[0] & 0x7;
	return (ResolutionExponent == 0u);
}

int3 GetBottomLevelVoxelResolution(FTopLevelGridData TopLevelGridData)
{
	uint ResolutionExponent = TopLevelGridData.PackedData[0] & 0x7;
	int3 Resolution = ResolutionExponent;

	return Resolution;
}

void SetBottomLevelVoxelResolution(inout FTopLevelGridData TopLevelGridData, int3 BottomLevelVoxelResolution)
{
	uint Index = GetBottomLevelIndex(TopLevelGridData);
	uint ResolutionExponent = BottomLevelVoxelResolution.x & 0x7;
	TopLevelGridData.PackedData[0] = (Index << 3) | ResolutionExponent;
}

float GetVoxelSize(FTopLevelGridData TopLevelGridData)
{
	return asfloat(TopLevelGridData.PackedData[0]);
}

void SetVoxelSize(inout FTopLevelGridData TopLevelGridData, float VoxelSize)
{
	TopLevelGridData.PackedData[0] = asint(VoxelSize);
}

// Convenience functions to retrieve the first element of a tile
float3 GetExtinction(FTopLevelGridData TopLevelGridData, StructuredBuffer<FScalarGridData> ExtinctionGridBuffer)
{
	float3 Extinction = GetExtinction(ExtinctionGridBuffer[GetBottomLevelIndex(TopLevelGridData)]);
	return Extinction;
}

float3 GetExtinction(FTopLevelGridData TopLevelGridData, StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer, StructuredBuffer<FScalarGridData> ExtinctionGridBuffer)
{
	uint IndirectionIndex = GetBottomLevelIndex(TopLevelGridData);
	uint BottomLevelIndex = GetBottomLevelIndex(IndirectionGridBuffer[IndirectionIndex]);
	float3 Extinction = GetExtinction(ExtinctionGridBuffer[BottomLevelIndex]);
	return Extinction;
}

float3 GetEmission(FTopLevelGridData TopLevelGridData, StructuredBuffer<FVectorGridData> EmissionGridBuffer)
{
	float3 Emission = GetEmission(EmissionGridBuffer[GetBottomLevelIndex(TopLevelGridData)]);
	return Emission;
}

float3 GetEmission(FTopLevelGridData TopLevelGridData, StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer, StructuredBuffer<FVectorGridData> EmissionGridBuffer)
{
	uint IndirectionIndex = GetBottomLevelIndex(TopLevelGridData);
	uint BottomLevelIndex = GetBottomLevelIndex(IndirectionGridBuffer[IndirectionIndex]);
	float3 Emission = GetEmission(EmissionGridBuffer[BottomLevelIndex]);
	return Emission;
}

float3 GetAlbedo(FTopLevelGridData TopLevelGridData, StructuredBuffer<FVectorGridData> ScatteringGridBuffer)
{
	float3 Albedo = GetAlbedo(ScatteringGridBuffer[GetBottomLevelIndex(TopLevelGridData)]);
	return Albedo;
}

float3 GetAlbedo(FTopLevelGridData TopLevelGridData, StructuredBuffer<FTopLevelGridData> IndirectionGridBuffer, StructuredBuffer<FVectorGridData> ScatteringGridBuffer)
{
	uint IndirectionIndex = GetBottomLevelIndex(TopLevelGridData);
	uint BottomLevelIndex = GetBottomLevelIndex(IndirectionGridBuffer[IndirectionIndex]);
	float3 Albedo = GetAlbedo(ScatteringGridBuffer[BottomLevelIndex]);
	return Albedo;
}
