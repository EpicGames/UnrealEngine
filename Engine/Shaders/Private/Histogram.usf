// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredShadingCommon.ush"
#include "BRDF.ush"
#include "PositionReconstructionCommon.ush"

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/PostProcessHistogramCommon.ush"
#include "/Engine/Private/PostProcessExposureMaterial.ush"

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//ATOMIC HISTOGRAM
//The main speedups are:
//	1) better read order which is cache friendly instead of cache thrashy.  The original code read a tile per thread which destroyed the l0 cache.
//		New one reads linearly (which isn't the best), but each thread read consecutive pixels so all the threads are on less cache lines.
//		It could probably be better if I read in a tile with each GROUP (not each thread!) reading from the same 8x8 tile, or even swizzle reading.
//		I'm profoundly ALU bound now, so MEH !
//	2) I loop through the pixels and each thread accumulates to an LDS Histogram. Depending on the shader model this can be done in two ways
//		a) If only 32 bit interlocked add to lds is supported I just write out this bin's value and the next bin's value using two interlocked adds.
//			We get some heavy serialization cost is the luminances are all the same, so (if available) I do some wave ops to detect if all waves have the same value
//		b) If 64 bit interlocked adds to lds are supported I pack and write both this bin and next bin's value to a single 64 bit LDS bin.  Later I unpack and split them up.
//			Note that this isn't faster in the average case, but mysteriously eliminates the heavy cost of serialization when all pixels have the same luminance.
//	3) At the end of the loop each threadgroup takes it's LDS and interlock add's it to a dispatch common texture
//		There are Two versions here too:
//		a) If we support 64 bit interlocked adds, then I add the histogram value using a 64 bit atomic interlocked add.  We need 64 bits because worst case is the
//			screen is one image, so the histogram value (which is fixed point) could be equal to the number of pixels (and I assumed 8k screen)
//		b) If we only support 32 bit interlocked adds, I simulated a 64 bit interlocked add by adding the low 32 bits, and if they overflow
//			I do a second interlocked add of the "carryover"


SCREEN_PASS_TEXTURE_VIEWPORT(Input)

// Number of thread groups in the dispatch
uint2 ThreadGroupCount;

// DO_64BIT_ATOMICS = Support For Doing 64 Bit Interlocked Adds to LDS
#define DO_64BIT_ATOMICS 0


// HISTOGRAMS_PER_THREAD is used to figure out how many histogram values each thread has to write.  Minimum is obviously at least 1
//  if for example you have a histogram size of 32 and threadgroup size of 64, then this is 1, but we do an "if thread < 32" and some threads go wasted.
//  if for example you have a histogram size of 64 and threadgroup size of 32, then this value is 2.  Each thread processes two histogram entries.
#define HISTOGRAMS_PER_THREAD ( (HISTOGRAM_SIZE < THREADGROUP_SIZEX) ? 1: HISTOGRAM_SIZE/THREADGROUP_SIZEX)

Texture2D InputTexture;
StructuredBuffer<float4> EyeAdaptationBuffer;

///////////////////////////////////////////////////////
// * LOCAL DATA STORE *
// This is where each ThreadGroup accumulates (via InterlockedAdd) a local (per thread group) copy of the histogram
#if DO_64BIT_ATOMICS
//OK So instead of two 32 bit writes, we do a single 64 bit packed write.  This isn't actually any faster in the average case, but is actually amazing
//   with the case where the luminances are all the same (or similar) and the atomics start to serialize.  ie. it doesn't even seem to be slower.
groupshared uint64_t PerThreadGroupHistogramLDS64[HISTOGRAM_SIZE];
#else
groupshared uint PerThreadGroupHistogramLDS32[HISTOGRAM_SIZE];
#endif

///////////////////////////////////////////////////////
//  * TEMPORARY TEXTURES *
// When each ThreadGroup is done the thread group uses interlocked add to add it's result to a Temporary UINT texture.
// This is actually a texture which is two pixels high.  The top line is the actual histogram.
// The second line (only the left most pixel) is used to count how many thread groups have run.

//globally coherant not technically needed for interlocked adds, but I DO read from the temp texture thus necessatting it theoretically.
//no perf hit either way.
#if DO_64BIT_ATOMICS
Texture2D<uint64_t> HistogramScatter64Texture;
RWTexture2D<uint64_t> HistogramScatter64Output;
#else
Texture2D<uint> HistogramScatter32Texture;
RWTexture2D<uint> HistogramScatter32Output;
#endif

///////////////////////////////////////////////////////
//  * FINAL OUTPUT *
//  We detect if we're the last thread group to run, if so, then we know that
//  we should pack up our Temp UINT textures, convert back to float and stick the histogram in the final texture
RWTexture2D<float4> HistogramOutput;

///////////////////////////////////////////////////////
//  * DEBUG TEXTURES *
RWTexture2D<float> DebugOutput;


///////////////////////////////////////////////////////
//  * PACKING HELPER FUNCTIONS *
// REGARDING PACKING FORMATS:
//OK.  so we're doing one horizontal line per thread-work group.
//  The numbers we are summing are all <= 1.0.
// @ 4k that's 3840 samples. Let's go crazy and say 8k support (7680 samples)!  Log2(7680) = 13 so we need 13 bits to sum it all up with no chance of overflowing
//  That means we can represent the histogram as 13.19 format.

#if DO_64BIT_ATOMICS
//Pack two floats into one 64 bit atomic.  This greatly reduces (completely?) serialization 
uint64_t PackTwoFloatsToUINT64(float f0, float f1)
{
	uint64_t a = (uint64_t)(f0 * float( 1<<19 ));
	uint64_t b = (uint64_t)(f1 * float( 1<<19 ));
	return ( (a<<32) | b);
}
void SplitUpperAndLowerUINT64IntoTwoUINT64(uint64_t v, out uint64_t l0, out uint64_t l1)
{
	l0 = (v>>32);
	l1 = v&0xffffffff;
}
#endif

uint PackFloatToUINT32(float v)
{
	return (uint)(v*float( 1<<19 ));
}
float UnpackUINT32ToFloat(uint v)
{
	float fv = (float)(v);
	return fv * (1.0f/float(1<<19));
}
float Unpack2UINT32ToFloat(uint hi, uint low)
{
	float flow = float(low) * (1.0f/float(1<<19));
	//so high 32 bits is effectively (hi<<32) * (1.0f/float(1<<19))
	float fhigh = float(hi) * float(1<<(32-19));
	return fhigh + flow;
}

#if DO_64BIT_ATOMICS
float SingleUINT64ToFloat(uint64_t v)
{
	float fv = (float)(v);
	return fv * (1.0f/float(1<<19));
}
#endif


///////////////////////////////////////////////////////
void ClearLDS(uint GroupThreadId)
{
	UNROLL 
	for (uint t = 0 ; t < HISTOGRAMS_PER_THREAD ; ++t)
	{
		uint index = GroupThreadId * HISTOGRAMS_PER_THREAD + t;
		if (index < HISTOGRAM_SIZE)
		{
			#if DO_64BIT_ATOMICS
				PerThreadGroupHistogramLDS64[index] = 0;
			#else
				PerThreadGroupHistogramLDS32[index] = 0;
			#endif
		}
	}
}

///////////////////////////////////////////////////////
//Given a Texel pos this reads in the screen value and calculates the two fractional bucket it lies in.
void CalculateBucketsAndWeights(in float LuminanceVal, out float Weight0, out float Weight1, out uint Bucket0, out uint Bucket1)
{
	float LogLuminance = ComputeHistogramPositionFromLuminance(LuminanceVal);
	// Map the normalized histogram position into texels.
	float fBucket = saturate(LogLuminance) * (HISTOGRAM_SIZE-1);

	// Find two discrete buckets that straddle the continuous histogram position.
	Bucket0 = (uint)(fBucket);
	Bucket1 = Bucket0 + 1;

	Bucket0 = min(Bucket0, uint(HISTOGRAM_SIZE - 1));
	Bucket1 = min(Bucket1, uint(HISTOGRAM_SIZE - 1));

	// Weighted blend between the two buckets.
	Weight1 = frac(fBucket);
	Weight0 = 1.0f - Weight1;

	// When EyeAdaptat=.0, we will ignore the last bucket. The main use
	// case is so the black background pixels in the editor have no effect. But if we have cases where
	// pixel values can actually be black, we want to set EyeAdaptation_LastHistogramBucketInfluence=1.0.
	// This value is controlled by the cvar "r.EyeAdaptation.BlackHistogramBucketInfluence"
	if (Bucket0 == 0)
	{
		Weight0 *= EyeAdaptation_BlackHistogramBucketInfluence;
	}
			
}

///////////////////////////////////////////////////////
//This will write the weights to the proper buckets in LDS
//This is called in an inner loop.
void ScatterWeightsToLDS(uint Bucket0, uint Bucket1, float Weight0, float Weight1, float ScreenWeight, uint GroupThreadId)
{
	#if DO_64BIT_ATOMICS
		// Accumulate the weight to the nearby history buckets.
		// oddly atomic serialization doesn't seem to be an issue at all when I use 64 bit atomics, so I don't bother with the wave 
		//    vote magic that I use in the 32 bit version.
		uint64_t w64 = PackTwoFloatsToUINT64(Weight0*ScreenWeight, Weight1 * ScreenWeight);
		//exciting 64 bit atomics
		InterlockedAdd(PerThreadGroupHistogramLDS64[Bucket0], w64);
	#else
		#if COMPILER_SUPPORTS_WAVE_VOTE
			// Accumulate the weight to the nearby history buckets.
			uint w;
			//handle (not so uncommon) case that the entire screen is one value,..ie all white, all black, or all grey.
			//check if every thread is the same, and if so just get one thread to add to LDS.
			//in this case, all those InterlockedAdds will try to add to the same bucket and get serialized, - Histogram gets about 61% slower. 
			// With this optimization an image which is one single luminance is an additional 2x faster instead of 61% slower.
			w = PackFloatToUINT32(Weight0*ScreenWeight);
			uint LaneCountPerWave = WaveGetActiveLaneIndexLast() + 1;
			//I only test Weight0 && Bucket 0  because Weight1 && Bucket1 are deterministically the same on all waves if Weight0 is the same on all waves
			bool AreAllLanesTheSame = WaveActiveAllTrue((ToScalarMemory(w) == w) && (ToScalarMemory(Bucket0) == Bucket0) );
			BRANCH
			if (AreAllLanesTheSame)
			{
				if (WaveGetLaneIndex() == 0 )
				{
					InterlockedAdd(PerThreadGroupHistogramLDS32[Bucket0], w*LaneCountPerWave);
					w = PackFloatToUINT32(Weight1*ScreenWeight);
					InterlockedAdd(PerThreadGroupHistogramLDS32[Bucket1], w*LaneCountPerWave);
				}
			}
			else
		#endif
		{
			//boring 32 bit atomics
			InterlockedAdd(PerThreadGroupHistogramLDS32[Bucket0], PackFloatToUINT32(Weight0*ScreenWeight));
			InterlockedAdd(PerThreadGroupHistogramLDS32[Bucket1], PackFloatToUINT32(Weight1*ScreenWeight));
		}
	#endif
}


///////////////////////////////////////////////////////
//Common Definitions for Interlocked Adds to textures.
//This is really just hear so we can call the same function regardless of 64 or 32 bit
#if DO_64BIT_ATOMICS 
	#define GatherType uint64_t
	void InterlockedAddToTexture(uint index, GatherType V)
	{
		InterlockedAdd(HistogramScatter64Output[uint2(index,0)],V);
	}
#else
	#define GatherType uint
	void InterlockedAddToTexture(uint index, GatherType V)
	{
		uint OldV;
		InterlockedAdd(HistogramScatter32Output[uint2(index*2 + 0 , 0)], V, OldV);
		BRANCH 
		if (0xffffffff - V < OldV) // if 0xffffffff < OldV + V then it overflowed
		{
			InterlockedAdd(HistogramScatter32Output[uint2(index*2 + 1, 0)], 1);
		}
	}
#endif


///////////////////////////////////////////////////////
// At the end of a thread group we basically copy out the values
// from PerThreadGroupHistogramLDS* and interlock add the values 
// to a distpatch-common texture HistogramScatter64Output*
void GatherWeightsFromLDSToTempTexture_Helper(uint GroupThreadId )
{
	GatherType Weight0=0;
	uint Bucket0 = GroupThreadId;
	#if DO_64BIT_ATOMICS 
		// if we're using 64 bit atomics we read out and unpack two values from this bucket - the value we recorded for this bucket, and the value to add to the next
		GatherType Weight1=0;
		SplitUpperAndLowerUINT64IntoTwoUINT64(PerThreadGroupHistogramLDS64[Bucket0], Weight0,Weight1);
		uint Bucket1 = min(Bucket0 + 1, uint(HISTOGRAM_SIZE - 1));  //consistent with original code
		BRANCH
		if (GroupThreadId < THREADGROUP_SIZEX) //dont need this if wavesize is 64 is histogram size.
		{
			InterlockedAddToTexture(Bucket1, Weight1);
		}
	#else
		Weight0 = PerThreadGroupHistogramLDS32[Bucket0];
	#endif
	
	InterlockedAddToTexture( Bucket0, Weight0 );
	
}

void GatherWeightsFromLDSToTempTexture(uint GroupThreadId)
{
	UNROLL
	for (uint t = 0 ; t < HISTOGRAMS_PER_THREAD ; ++t)
	{
		uint index = GroupThreadId * HISTOGRAMS_PER_THREAD + t;
		if (index < HISTOGRAM_SIZE)
		{
			GatherWeightsFromLDSToTempTexture_Helper(index);
		}
	}

}

///////////////////////////////////////////////////////
// ENTRY POINT
[numthreads(THREADGROUP_SIZEX, 1, 1)]
void MainAtomicCS(
	uint3 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	//////////////////////////////////////////////////////
	// * CLEAR LDS *
	ClearLDS(GroupThreadId.x);
	
	GroupMemoryBarrierWithGroupSync();

	//////////////////////////////////////////////////////
	// * ACCUMULATE HISTOGRAM FOR ENTIRE LINE IN LDS *  (SCATTER)
	uint y = GroupId.x;
	for (uint x = 0 ; x < Input_ViewportSize.x ; x+=THREADGROUP_SIZEX)
	{
		uint2 TexelPos = uint2(x + GroupThreadId.x, y) + Input_ViewportMin;

		if (TexelPos.x < Input_ViewportMax.x)
		{
			const float2 InvViewSize = Input_ViewportSizeInverse.xy;
			const float2 ScreenUV = (TexelPos.xy - Input_ViewportMin + 0.5f) * InvViewSize;

			float3 SceneColor = InputTexture.Load(int3(TexelPos, 0)).rgb;
			SceneColor *= View.OneOverPreExposure;

			float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor);

#if USE_PRECALCULATED_LUMINANCE
			LuminanceVal = SceneColor.r;
#elif USE_APPROX_ILLUMINANCE
			const uint2 GBufferTexelPos = View.ViewRectMinAndSize.xy + ScreenUV * View.ViewRectMinAndSize.zw;
			
			const FExposureMaterial Material = GetExposureMaterial(GBufferTexelPos);
			if (Material.bIsValid)
			{
				float BaseColorLuminanceVal = CalculateEyeAdaptationLuminance(Material.DirectionalAlbedo);

				LuminanceVal /= max(BaseColorLuminanceVal, EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance);
				LuminanceVal *= lerp(EyeAdaptation_IgnoreMaterialsLuminanceScale, 1.0f, EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance);
			}
#endif

#if USE_DEBUG_OUTPUT
			DebugOutput[TexelPos - Input_ViewportMin] = LuminanceVal;
#endif

			float Weight0,Weight1;
			uint Bucket0,Bucket1;
			
			CalculateBucketsAndWeights(LuminanceVal, Weight0, Weight1, Bucket0, Bucket1);

			float ScreenWeight = AdaptationWeightTexture(ScreenUV);

			ScatterWeightsToLDS(Bucket0, Bucket1, Weight0, Weight1, ScreenWeight, GroupThreadId.x);
		}
	}


    GroupMemoryBarrierWithGroupSync();


	//////////////////////////////////////////////////////
	// * UNPACK LDS TO BUFFER WITH ATOMICS (once per thread group)  (GATHER)
	GatherWeightsFromLDSToTempTexture(GroupThreadId.x);

}



///second pass to convert to legacy format
[numthreads(THREADGROUP_SIZEX, 1, 1)]
void HistogramConvertCS(uint3 GroupThreadId : SV_GroupThreadID)
{
	float2 InvViewSize = Input_ViewportSizeInverse.xy;
	float NormalizeFactor = InvViewSize.x * InvViewSize.y;
	float4 OldExposureScale = ToScalarMemory(EyeAdaptationBuffer[0]);

	UNROLL_N(HISTOGRAMS_PER_THREAD)
	for (uint t = 0; t < HISTOGRAMS_PER_THREAD; t++)
	{
		uint index = GroupThreadId.x*HISTOGRAMS_PER_THREAD + t;
		BRANCH
		if (index < HISTOGRAM_SIZE /4)
		{
			#if DO_64BIT_ATOMICS 
				float4 Val = float4(
					SingleUINT64ToFloat(HistogramScatter64Texture[uint2(index * 4 + 0, 0)]),
					SingleUINT64ToFloat(HistogramScatter64Texture[uint2(index * 4 + 1, 0)]),
					SingleUINT64ToFloat(HistogramScatter64Texture[uint2(index * 4 + 2, 0)]),
					SingleUINT64ToFloat(HistogramScatter64Texture[uint2(index * 4 + 3, 0)]));
			#else
				uint4 ValLow = uint4(
					HistogramScatter32Texture[uint2((index * 4 + 0) * 2 + 0, 0)], 	
					HistogramScatter32Texture[uint2((index * 4 + 1) * 2 + 0, 0)],
					HistogramScatter32Texture[uint2((index * 4 + 2) * 2 + 0, 0)],
					HistogramScatter32Texture[uint2((index * 4 + 3) * 2 + 0, 0)]);
				uint4 ValHigh = uint4(
					HistogramScatter32Texture[uint2((index * 4 + 0) * 2 + 1, 0)], 	
					HistogramScatter32Texture[uint2((index * 4 + 1) * 2 + 1, 0)],
					HistogramScatter32Texture[uint2((index * 4 + 2) * 2 + 1, 0)],
					HistogramScatter32Texture[uint2((index * 4 + 3) * 2 + 1, 0)]);
				//can't shift up by 32 in a 32 bit value
				float4 Val;
				Val.x = Unpack2UINT32ToFloat(ValHigh.x, ValLow.x);
				Val.y = Unpack2UINT32ToFloat(ValHigh.y, ValLow.y);
				Val.z = Unpack2UINT32ToFloat(ValHigh.z, ValLow.z);
				Val.w = Unpack2UINT32ToFloat(ValHigh.w, ValLow.w);
			#endif

			//what ?  why multiply by 0.5 ?  Well because this code snippet replaces PostProcessHistogramReduce which uses bilinear filtering, and effetively reduces the count by a factor of 2
			Val *= 0.5f; 
			HistogramOutput[uint2(index, 0)] = Val * NormalizeFactor;
			HistogramOutput[uint2(index, 1)] = OldExposureScale;
		}
	}
}

