// Copyright Epic Games, Inc. All Rights Reserved.


#include "LensDistortion.ush"
#include "ScreenPass.ush"


Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;
Texture2D HitProxyTexture;

FScreenTransform PassSvPositionToViewportUV;
FScreenTransform ViewportUVToHitProxyPixelPos;
int2 HitProxyPixelPosMin;
int2 HitProxyPixelPosMax;


void MainPS(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 DistortedViewportUV = ApplyScreenTransform(SvPosition.xy, PassSvPositionToViewportUV);
	float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);

	int2 HitProxyPixelPos = int2(ApplyScreenTransform(UndistortedViewportUV, ViewportUVToHitProxyPixelPos));
	HitProxyPixelPos = clamp(HitProxyPixelPos, HitProxyPixelPosMin, HitProxyPixelPosMax);

	OutColor = HitProxyTexture[HitProxyPixelPos];
;}
