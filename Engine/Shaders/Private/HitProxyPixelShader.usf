// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HitProxyPixelShader.hlsl: Pixel shader for rendering scene hit proxies.
=============================================================================*/

// Some input nodes can't compute their output value at hit proxy rendering time, and so their implementation changes.
#define HIT_PROXY_SHADER 1

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "/Engine/Private/BasePassCommonFunctions.ush"

// Whether HitProxy IDs are provided per instance or not
#define PER_INSTANCE_HITPROXY_ID	(USE_INSTANCING || USE_PER_VERTEX_HITPROXY_ID || USE_VERTEXFACTORY_HITPROXY_ID)

float4 HitProxyId;
float WriteInstanceHitProxyId; // 1.0 or 0.0 to select whether or not per-instance ID is added to the output

void Main(
	FVertexFactoryInterpolantsVSToPS Interpolants,
	float4 InstanceHitProxyId : HIT_PROXY_ID,
	float4 PixelPosition	: TEXCOORD6,
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO : TEXCOORD7,
#endif
	in FStereoPSInput StereoInput,
	in INPUT_POSITION_QUALIFIERS float4 SvPosition	: SV_Position
	OPTIONAL_IsFrontFace
	OPTIONAL_OutDepthConservative
	,out float4 OutColor		: SV_Target0
	)
{
	StereoSetupPS(StereoInput);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
	CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, PixelPositionExcludingWPO);	
#else
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
#endif

	#if OUTPUT_PIXEL_DEPTH_OFFSET
		ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);
	#endif

	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

	OutColor = HitProxyId + InstanceHitProxyId * WriteInstanceHitProxyId;
}
