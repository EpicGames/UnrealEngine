// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HitProxyVertexShader.hlsl: Vertex shader for rendering hit proxies.
=============================================================================*/

// Some input nodes can't compute their output value at hit proxy rendering time, and so their implementation changes.
#define HIT_PROXY_SHADER 1

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#if USE_PER_VERTEX_HITPROXY_ID
Buffer<float4> VertexFetch_HitProxyIdBuffer;
#endif

#ifndef VF_REQUIRES_HITPROXY_INDIRECTION 
#define VF_REQUIRES_HITPROXY_INDIRECTION 0
#endif

#if VF_REQUIRES_HITPROXY_INDIRECTION == 0
	int VertexFactoryGetVertexFetchParameter(int ParameterIndex)
	{
		return 0;
	}
#endif

struct FHitProxyVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 InstanceHitProxyId : HIT_PROXY_ID;
	float4 PixelPosition : TEXCOORD6;
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO : TEXCOORD7;
#endif
	FStereoVSOutput StereoOutput;
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
};

#define FHitProxyVSOutput FHitProxyVSToPS

#if VERTEXSHADER

void Main(
	FVertexFactoryInput Input,
	out FHitProxyVSOutput Output
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, Output.StereoOutput);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 WorldPositionExcludingWPO = WorldPosition;
	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
	Output.InstanceHitProxyId = VertexFactoryGetInstanceHitProxyId(Input, VFIntermediates);
#if USE_PER_VERTEX_HITPROXY_ID
	#if MANUAL_VERTEX_FETCH
		Output.InstanceHitProxyId = VertexFetch_HitProxyIdBuffer[(VertexFactoryGetVertexFetchParameter(VF_VertexOffset) + Input.VertexId)] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	#else
		Output.InstanceHitProxyId = float4(0,0,0,0);
	#endif
#endif

	{
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
		Output.Position = INVARIANT(mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip));
	}

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
	Output.PixelPosition = WorldPosition;

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	Output.PixelPositionExcludingWPO = WorldPositionExcludingWPO.xyz;
#endif
}

#endif // VERTEXSHADER
