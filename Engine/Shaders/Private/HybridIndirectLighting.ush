// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)

#include "Common.ush"
#include "SceneTextureParameters.ush"
#include "Random.ush"
#include "BRDF.ush"
#include "MonteCarlo.ush"


/** Size of the interleaving tile, 4x4. */
#define INDIRECT_LIGHTING_INTERLEAVING_TILE_SIZE 4

/** Total number of bucked used to interleave. */
#define INDIRECT_LIGHTING_INTERLEAVING_BUCKET_COUNT (INDIRECT_LIGHTING_INTERLEAVING_TILE_SIZE * INDIRECT_LIGHTING_INTERLEAVING_TILE_SIZE)

/** Maximum number of rays that can be shot per ray tracing pixel. */
#define INDIRECT_LIGHTING_MAX_RAY_PER_PIXEL 8

/** Maximum resolution of rays ray tracing pixel 8192x8192. */
#define INDIRECT_LIGHTING_MAX_TRACING_RESOLUTION 8192


/** FCommonDiffuseIndirectParameters */
uint2 TracingViewportSize;
uint2 TracingViewportBufferSize;
float2 TracingViewportTexelSize;
uint DownscaleFactor;
uint RayCountPerPixel;
uint2 RayStoragePerPixelVector;
uint PixelRayIndexAbscissMask;
uint PixelRayIndexOrdinateShift;

uint PackRayInformationToGlobalRayId(uint2 TracingPixelCoord, uint PixelRayIndex)
{
	uint GlobalRayIndex = ((TracingPixelCoord.x <<  0) | (TracingPixelCoord.y << 13) | (PixelRayIndex << 26));
	return GlobalRayIndex;
}

// Inverse of PackRayInformationToGlobalRayId()
void UnpackRayInformationFromGlobalRayId(uint GlobalRayIndex, out uint2 TracingPixelCoord, out uint PixelRayIndex)
{
	TracingPixelCoord.x = (GlobalRayIndex >>  0) & 0x1FFF;
	TracingPixelCoord.y = (GlobalRayIndex >> 13) & 0x1FFF;
	PixelRayIndex = (GlobalRayIndex >> 26) & 0x7;
}

uint2 GetRayStorageCoords(uint2 TracingPixelCoord, uint PixelRayIndex)
{
	uint2 PixelRayVector = uint2(PixelRayIndex & PixelRayIndexAbscissMask, PixelRayIndex >> PixelRayIndexOrdinateShift);

	//return TracingPixelCoord + PixelRayVector * TracingViewportBufferSize;
	return TracingPixelCoord * RayStoragePerPixelVector + PixelRayVector;
}

float2 TracingPixelCoordToSceneBufferUV(uint2 TracingPixelCoord)
{
	// TODO(Guillaume): Standard lower resolution.
	return (TracingPixelCoord * DownscaleFactor + View.ViewRectMin.xy + 0.5) * View.BufferSizeAndInvSize.zw;
}
