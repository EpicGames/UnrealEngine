// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ShaderPrint.ush"
#include "ColorMap.ush"

//////////////////////////////////////////////////////////////////////////////////////////////// 
// Add texture
#if SHADER_ADD_TEXTURE_CS

Texture2D<float4> InTexture_0;
Texture2D<float4> InTexture_1;
Texture2D<float4> InTexture_2;
Texture2D<float4> InTexture_3;
Texture2D<float4> InTexture_4;
Texture2D<float4> InTexture_5;
Texture2D<float4> InTexture_6;
Texture2D<float4> InTexture_7;

uint4 InSliceIndex[8];

SamplerState InSampler_0;
SamplerState InSampler_1;
SamplerState InSampler_2;
SamplerState InSampler_3;
SamplerState InSampler_4;
SamplerState InSampler_5;
SamplerState InSampler_6;
SamplerState InSampler_7;

int2 AtlasResolution;
uint AtlasSliceCount;
uint ValidCount;

RWTexture2DArray<float> OutAtlasTexture;

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId.xy < uint2(AtlasResolution)))
	{
		const uint2 DstPixelPos = DispatchThreadId.xy;
		uint DstSlice = 0;

		const float2 SrcUV = (DstPixelPos + 0.5) / float2(AtlasResolution);
		const uint SrcSlice = DispatchThreadId.z;

		if (SrcSlice < ValidCount)
		{
			float Color = 0;
			switch (SrcSlice)
			{
			case 0: Color = InTexture_0.SampleLevel(InSampler_0, SrcUV, 0).x; DstSlice = InSliceIndex[0].x; break;
			case 1: Color = InTexture_1.SampleLevel(InSampler_1, SrcUV, 0).x; DstSlice = InSliceIndex[1].x; break;
			case 2: Color = InTexture_2.SampleLevel(InSampler_2, SrcUV, 0).x; DstSlice = InSliceIndex[2].x; break;
			case 3: Color = InTexture_3.SampleLevel(InSampler_3, SrcUV, 0).x; DstSlice = InSliceIndex[3].x; break;
			case 4: Color = InTexture_4.SampleLevel(InSampler_4, SrcUV, 0).x; DstSlice = InSliceIndex[4].x; break;
			case 5: Color = InTexture_5.SampleLevel(InSampler_5, SrcUV, 0).x; DstSlice = InSliceIndex[5].x; break;
			case 6: Color = InTexture_6.SampleLevel(InSampler_6, SrcUV, 0).x; DstSlice = InSliceIndex[6].x; break;
			case 7: Color = InTexture_7.SampleLevel(InSampler_7, SrcUV, 0).x; DstSlice = InSliceIndex[7].x; break;
			}

			// Ensure there is no NaN value
			Color = -min(-Color, 0);

			DstSlice = min(DstSlice, AtlasSliceCount-1);
			OutAtlasTexture[uint3(DstPixelPos, DstSlice)] = Color;
		}
	}
}

#endif // SHADER_ADD_TEXTURE_CS

#if SHADER_ADD_TEXTURE_PS

Texture2D<float4> InTexture;
SamplerState InSampler;
int2 AtlasResolution;

void MainPS(float4 SvPosition : SV_POSITION,
			out float OutColor : SV_Target0)
{
	const float2 SrcUV = SvPosition.xy / float2(AtlasResolution);
	float Color = InTexture.SampleLevel(InSampler, SrcUV, 0).x;
	// Ensure there is no NaN value
	OutColor = -min(-Color, 0);
}

#endif // SHADER_ADD_TEXTURE_PS

////////////////////////////////////////////////////////////////////////////////////////////////
// Debug
#if SHADER_DEBUG

int2 AtlasResolution;
uint AtlasSliceCount;
uint TotalSlotCount;
uint ValidSlotCount;
uint UsedSliceCount;
uint bForceUpdate;

int2 OutputResolution;

Buffer<uint> ValidSliceBuffer;

SamplerState AtlasSampler;
Texture2DArray<float4> AtlasTexture;
RWTexture2D<float4> OutputTexture;

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

bool IsInSlot(float2 Coord, float2 MinRect, float2 MaxRect)
{
	return all(Coord >= MinRect) && all(Coord <= MaxRect);
}

bool IsOnBorder(float2 Coord, float2 MinRect, float2 MaxRect, float BorderSize)
{
	const bool bIsWithin = IsInSlot(Coord, MinRect, MaxRect);
	const bool bIsOnBorder = any(Coord - MinRect <= BorderSize) || any(MaxRect - Coord  <= BorderSize);
	return bIsWithin && bIsOnBorder;
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw stats
	if (all(DispatchThreadId == 0))
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 100));

		Print(Context, TEXT("Atlas resolution : "), FontSilver);
		Print(Context, AtlasResolution.x, FontOrange, 3, 0);
		Print(Context, TEXT("x"), FontSilver);
		Print(Context, AtlasResolution.y, FontOrange, 3, 0);
		Newline(Context);

		Print(Context, TEXT("Atlas slice      : "), FontSilver);
		Print(Context, UsedSliceCount, FontOrange, 3, 0);
		Print(Context, TEXT("/"), FontSilver);
		Print(Context, AtlasSliceCount, FontOrange, 3, 0);
		Newline(Context);

		const float Occupancy = UsedSliceCount / float(AtlasSliceCount);
		const FFontColor OccupancyColor = GetOccupancyColor(Occupancy);
		Print(Context, TEXT("Atlas occupancy  : "), FontSilver);
		Print(Context, Occupancy * 100.f, OccupancyColor, 4, 1);
		Print(Context, TEXT("% "), FontSilver);
		Newline(Context);

		Print(Context, TEXT("Slot count       : "), FontSilver);
		Print(Context, ValidSlotCount, FontOrange);
		Newline(Context);

		if (TotalSlotCount > AtlasSliceCount)
		{
			Print(Context, TEXT("Slot over budget : "), FontSilver);
			Print(Context, TotalSlotCount - AtlasSliceCount, FontRed, 3, 0);
			Print(Context, TEXT(" - "), FontRed);
			Print(Context, TotalSlotCount, FontRed, 3, 0);
			Print(Context, TEXT("/"), FontRed);
			Print(Context, AtlasSliceCount, FontRed, 3, 0);
			Print(Context, TEXT(" "), FontRed);
			Newline(Context);
		}

		Print(Context, TEXT("Force Update     : "), FontSilver);
		if (bForceUpdate)
		{
			Print(Context, TEXT("True"), FontGreen);
		}
		else
		{
			Print(Context, TEXT("False"), FontRed);
		}
		Newline(Context); 


		Print(Context, TEXT("Atlas format     : "), FontSilver);
		Print(Context, TEXT("PF_R16"), FontOrange);
		Newline(Context);
	}

	// Draw atlas
	const uint2 GridRes = ceil(sqrt(AtlasSliceCount)).xx;
	const uint2 SlotResolution = AtlasResolution / GridRes;
	const uint2 Coord = DispatchThreadId.xy;
	const float2 OutputUV = float2(DispatchThreadId.xy) / float2(OutputResolution);
	const float AtlasRatio  = AtlasResolution.y / float(AtlasResolution.x);
	const float OutputRatio = OutputResolution.y / float(OutputResolution.x);

	const float ScaleFactor = 0.25f;
	const uint2 DisplayAtlasOffset     = uint2(50, 220);
	const uint2 DisplayAtlasResolution = uint2(OutputResolution.x * ScaleFactor, OutputResolution.x * ScaleFactor * AtlasResolution.y / AtlasResolution.x);
	if (all(Coord > DisplayAtlasOffset) && all(Coord < DisplayAtlasOffset + DisplayAtlasResolution))
	{
		const float2 AtlasUV	= (Coord - DisplayAtlasOffset) / float2(DisplayAtlasResolution);
		const float2 AtlasCoord = AtlasUV * AtlasResolution;
		if (all(AtlasUV < 1.0f))
		{
			const float BorderSize =  float(AtlasResolution.x) / float(DisplayAtlasResolution.x) * 2.f;
			const float2 IESSlotUV = frac(AtlasUV * GridRes);
			const uint   IESIndex  = floor(AtlasUV.x * GridRes.x) + floor(AtlasUV.y * GridRes.y) * GridRes.x;
			float4 Color = 0;
			if (IESIndex < AtlasSliceCount)
			{
				const bool bIsValid = ValidSliceBuffer[IESIndex] > 0;
				if (bIsValid)
				{
					Color = AtlasTexture.SampleLevel(AtlasSampler, float3(IESSlotUV, IESIndex), 0);
				}

				if (IsOnBorder(IESSlotUV * SlotResolution, float2(0,0), SlotResolution.xx, BorderSize))
				{
					Color = bIsValid ? float4(0, 1, 0, 1) : float4(0.5, 0.5, 0.5, 1);
				}
			}

			OutputTexture[DispatchThreadId.xy] = float4(Color.xyz, 1.0f);
		}
	}
	else
	{
		OutputTexture[DispatchThreadId.xy] = 0.f;
	}
}

#endif // SHADER_DEBUG
