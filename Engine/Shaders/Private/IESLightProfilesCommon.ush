// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IESLightProfiles.ush: Common shader code for IES light profiles
=============================================================================*/

#if (RAYHITGROUPSHADER || RAYGENSHADER || RAYMISSSHADER)
	#define USE_IES_PROFILE 1
#else
  #ifndef USE_IES_PROFILE
	#define USE_IES_PROFILE 0
  #endif
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define IESAtlasSampler	View.SharedBilinearClampedSampler
#else
#define IESAtlasSampler	View.IESAtlasSampler
#endif

// Special case for IES texture preview, which require a standalone texture
#if defined(USE_IES_STANDALONE_TEXTURE) && USE_IES_STANDALONE_TEXTURE == 1
Texture2D		IESTexture;
SamplerState	IESTextureSampler;
#endif

// Apply IES light profile texture
float ComputeLightProfileMultiplier(float3 WorldPosition, float3 LightPosition, float3 LightDirection, float3 LightTangent, float InIESAtlasIndex)
{
	float Out = 1.0f;
#if USE_IES_PROFILE && !VISUALIZE_LIGHT_CULLING

#if !defined(USE_IES_STANDALONE_TEXTURE) || !USE_IES_STANDALONE_TEXTURE
	if (InIESAtlasIndex >= 0)
#endif
	{
		float3 LightBitangent = normalize( cross( LightTangent, LightDirection ) );

		float4x4 LightTransform = float4x4( float4(LightDirection.xyz, 0), float4(LightBitangent.xyz, 0), float4(LightTangent.xyz, 0), float4(0, 0, 0, 1) );
		float4x4 InvLightTransform = transpose(LightTransform);

		float3 ToLight = normalize(LightPosition - WorldPosition);
		float3 LocalToLight = mul(float4(ToLight.xyz, 0), InvLightTransform).xyz;

		// -1..1
		float DotProd = dot(ToLight, LightDirection);
		// -PI..PI (this distortion could be put into the texture but not without quality loss or more memory)
		float Angle = asin(DotProd);
		// 0..1
		float NormAngle = Angle / PI + 0.5f;

		float TangentAngle = atan2( -LocalToLight.z, -LocalToLight.y ); // -Y represents 0/360 horizontal angle and we're rotating counter-clockwise
		float NormTangentAngle = TangentAngle / (PI * 2.f) + 0.5f;

	#if defined(USE_IES_STANDALONE_TEXTURE) && USE_IES_STANDALONE_TEXTURE == 1
		Out = Texture2DSampleLevel(IESTexture, IESTextureSampler, float2(NormAngle, NormTangentAngle), 0).r;
	#else
		Out = View.IESAtlasTexture.SampleLevel(IESAtlasSampler, float3(NormAngle, NormTangentAngle, InIESAtlasIndex), 0);
	#endif
	}
#endif
	return Out;
}
