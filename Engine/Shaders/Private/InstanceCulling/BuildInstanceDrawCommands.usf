// Copyright Epic Games, Inc. All Rights Reserved.

// Just used to trigger shader recompiles, guid should be regenerated on merge conflicts
#pragma message("UESHADERMETADATA_VERSION 5F4355AE-A326-40DB-99A4-0666E9076245")

#ifndef ENABLE_PER_INSTANCE_CUSTOM_DATA
	#define ENABLE_PER_INSTANCE_CUSTOM_DATA 1
#endif

#if ENABLE_INSTANCE_COMPACTION
	// Currently, instance compaction is the only reason to need extended payload data, so tie it to the permutation for now.
	#define ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS 1
#endif

#ifndef INSTANCE_DATA_STRIDE_ELEMENTS
	#define INSTANCE_DATA_STRIDE_ELEMENTS 1u
#endif

#ifndef VF_SUPPORTS_PRIMITIVE_SCENE_DATA
	// Enable access to SceneData functionality for these compute shaders
	#define VF_SUPPORTS_PRIMITIVE_SCENE_DATA 1u
#endif

// Disable the declaration of any templated types so we can use this shader without HLSL 2021. This is necessary because
// cross-compilation creates bugs with ENABLE_INSTANCE_COMPACTION.
#define ALLOW_TEMPLATES 0

// Do not use shared samplers as it requires the View uniform buffer, which is not bound for this shader.
#define USE_HZB_SHARED_SAMPLERS 0

#include "../Common.ush"
#include "../SceneData.ush"
#include "../LightmapData.ush"
#include "../DynamicMeshBounds.ush"
#include "InstanceCullingCommon.ush"
#include "InstanceCompactionCommon.ush"

// Turn on the logic for culling based on min screen radius (used in NaniteCullingCommon.ush)
#define NANITE_CULLING_ENABLE_MIN_RADIUS_CULL 1
#include "../Nanite/NaniteCullingCommon.ush"
#include "../WaveOpUtil.ush"
#include "../ComputeShaderUtils.ush"

#if SINGLE_INSTANCE_MODE
	// Enable a load balancer optimization where all items are expected to have a single instance
	#define LOAD_BALANCER_SINGLE_INSTANCE_MODE 1
#endif


#include "InstanceCullingLoadBalancer.ush"
#include "InstanceCullingSetup.ush"

StructuredBuffer<uint2> DrawCommandDescs;
Buffer<uint> InstanceIdOffsetBuffer;
StructuredBuffer<uint> ViewIds;

uint NumViewIds;
uint DynamicInstanceIdOffset;
uint DynamicInstanceIdMax;
uint NumCullingViews;
uint CurrentBatchProcessingMode;
uint bDrawOnlySelected;

RWStructuredBuffer<uint> InstanceIdsBufferOut;
RWStructuredBuffer<float4> InstanceIdsBufferOutMobile;
RWBuffer<uint> DrawIndirectArgsBufferOut;

// Used for draw commands that need to use compaction to preserve instance order
StructuredBuffer<FPackedDrawCommandCompactionData> DrawCommandCompactionData;
RWStructuredBuffer<uint> CompactInstanceIdsBufferOut;
RWStructuredBuffer<uint> CompactionBlockCounts;

// uint per instance with 1 bit per view identified by NaniteView::InstanceOcclusionQueryMask
Buffer<uint> InstanceOcclusionQueryBuffer;

// this is just to avoid compiling packing code for unrelated shaders
#if PLATFORM_USES_PRIMITIVE_UBO
#include "../SceneDataMobileWriter.ush" 

void WriteDataUBO(RWStructuredBuffer<float4> Output, uint Offset, uint InstanceId, FPrimitiveSceneData PrimitiveData, FInstanceSceneData InstanceData, uint MeshLODIndex)
{
#if SINGLE_INSTANCE_MODE
	WritePrimitiveDataUBO(Output, Offset, PrimitiveData, MeshLODIndex);
#else
	WriteInstanceDataUBO(Output, Offset, InstanceId, InstanceData);
#endif
}
#endif //PLATFORM_USES_PRIMITIVE_UBO

void WriteInstance(uint Offset, uint InstanceId, FPrimitiveSceneData PrimitiveData, FInstanceSceneData InstanceData, uint ViewIdIndex, uint CullingFlags, uint MeshLODIndex)
{
	checkSlow(InstanceId < GetSceneData().MaxAllocatedInstanceId);

#if PLATFORM_USES_PRIMITIVE_UBO
	WriteDataUBO(InstanceIdsBufferOutMobile, Offset, InstanceId, PrimitiveData, InstanceData, MeshLODIndex);
#else
	uint PackedId = PackInstanceCullingOutput(InstanceId, ViewIdIndex, CullingFlags);
	checkStructuredBufferAccessSlow(InstanceIdsBufferOut, Offset);
	InstanceIdsBufferOut[Offset] = PackedId;
#endif
}

void WriteInstanceForCompaction(uint Offset, bool bVisible, uint InstanceId, FInstanceSceneData InstanceData, uint ViewIdIndex, uint CullingFlags)
{
	checkSlow(InstanceId < GetSceneData().MaxAllocatedInstanceId);

	uint PackedId = bVisible ? PackInstanceCullingOutput(InstanceId, ViewIdIndex, CullingFlags) : 0xFFFFFFFFU;
	checkStructuredBufferAccessSlow(CompactInstanceIdsBufferOut, Offset);
	CompactInstanceIdsBufferOut[Offset] = PackedId;	
}

float RoundUpF16(float DeviceZ)
{
	return f16tof32(f32tof16(DeviceZ) + 1);
}

bool IsInstanceVisible(FPrimitiveSceneData PrimitiveData, FInstanceSceneData InstanceData, uint InstanceId, uint ViewIdIndex, bool bAllowOcclusionCulling, bool bDrawOnlySelected, FDrawCommandDesc DrawCommandDesc, inout uint CullingFlags)
{
	CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;

#if CULL_INSTANCES
	// Should not draw invalid instances, 
	if (!InstanceData.ValidInstance)
	{
		return false;
	}

#if USE_EDITOR_SHADERS
	bool bHasPerInstanceEditorData = (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA) != 0u;
	if (bDrawOnlySelected && bHasPerInstanceEditorData && !InstanceData.EditorData.bIsSelected)
	{
		return false;
	}
#endif
	// TODO: The test for dot(InstanceData.LocalBoundsExtent, InstanceData.LocalBoundsExtent) <= 0.0f is just a workaround since the FDynamicMeshBuilder::GetMesh
	//       seems to just set empty bounds (and FLineBatcherSceneProxy pretends everything is at the origin). In the future these should compute reasonable bounds and 
	//       this should be removed.
	if (dot(InstanceData.LocalBoundsExtent, InstanceData.LocalBoundsExtent) <= 0.0f)
	{
		return true;
	}
#elif ALLOW_WPO_DISABLE
	// When culling is disabled, we don't need to do anything if we don't have to evaluate WPO disable distance
	if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE) == 0)
	{
		return true;
	}
#endif

#if CULL_INSTANCES || ALLOW_WPO_DISABLE
	// TODO: remove this indirection and go straight to data index
	checkStructuredBufferAccessSlow(ViewIds, ViewIdIndex);

	float3 LocalBoundsCenter = InstanceData.LocalBoundsCenter;
	float3 LocalBoundsExtent = InstanceData.LocalBoundsExtent;
	LoadDynamicMeshBounds(DrawCommandDesc.DynamicMeshBoundsIndex, LocalBoundsCenter, LocalBoundsExtent);

	uint ViewDataIndex = ViewIds[ViewIdIndex];

	if (ViewDataIndex < NumCullingViews)
	{
		FNaniteView NaniteView = GetNaniteView(ViewDataIndex);
		FInstanceDynamicData DynamicData = CalculateInstanceDynamicData(NaniteView, InstanceData, true);

		FBoxCull Cull;
		Cull.Init(
			NaniteView,
			LocalBoundsCenter,
			LocalBoundsExtent,
			InstanceData.NonUniformScale,
			DynamicData.LocalToTranslatedWorld,
			DynamicData.PrevLocalToTranslatedWorld );

#if !CULL_INSTANCES
		Cull.bDistanceCull				= false;
		Cull.bSkipDrawDistance			= true;
		Cull.bSkipCullGlobalClipPlane	= true;
#endif

#if ALLOW_WPO_DISABLE
		Cull.bSkipWPODisableDistance	|= DrawCommandDesc.bMaterialAlwaysEvaluatesWorldPositionOffset;
#else
		Cull.bSkipWPODisableDistance	= true;
#endif

		Cull.Distance( PrimitiveData );
		if (!Cull.bEnableWPO)
		{
			CullingFlags &= ~INSTANCE_CULLING_FLAG_EVALUATE_WPO;
		}

#if CULL_INSTANCES
		Cull.ScreenSize(DrawCommandDesc.MinScreenSize, DrawCommandDesc.MaxScreenSize);

		Cull.GlobalClipPlane();

		BRANCH
		if( Cull.bIsVisible )
		{
			Cull.Frustum();
		}

#if OCCLUSION_CULL_INSTANCES
		BRANCH
		if (Cull.bIsVisible && bAllowOcclusionCulling)
		{
			const bool bPrevIsOrtho = IsOrthoProjection(NaniteView.PrevViewToClip);
			FFrustumCullData PrevCull = BoxCullFrustum(LocalBoundsCenter, LocalBoundsExtent, DynamicData.PrevLocalToTranslatedWorld, NaniteView.PrevTranslatedWorldToClip, NaniteView.PrevViewToClip, bPrevIsOrtho, Cull.bNearClip, true);
			BRANCH
			if (PrevCull.bIsVisible && !PrevCull.bCrossesNearPlane)
			{
				FScreenRect PrevRect = GetScreenRect( NaniteView.HZBTestViewRect, PrevCull, 4 );
				// Avoid cases where instance might self-occlude the HZB test due to minor precision differences
				PrevRect.Depth = RoundUpF16(PrevRect.Depth);
				Cull.bIsVisible = IsVisibleHZB( PrevRect, true );
			}

			BRANCH
			if (NaniteView.InstanceOcclusionQueryMask && Cull.bIsVisible)
			{
				if ((InstanceOcclusionQueryBuffer[InstanceId] & NaniteView.InstanceOcclusionQueryMask) == 0)
				{
					Cull.bIsVisible = false;
				}
			}
		}
#endif // OCCLUSION_CULL_INSTANCES
#endif // CULL_INSTANCES

		return Cull.bIsVisible;
	}
#endif // CULL_INSTANCES || ALLOW_WPO_DISABLE

	return true;
}


/**
 * Each thread loops over a range on instances loaded from a buffer. The instance bounds are projected to all cached virtual shadow map address space
 * and any overlapped pages are marked as invalid.
 */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void InstanceCullBuildInstanceIdBufferCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	uint DispatchGroupId = GetUnWrappedDispatchGroupId(GroupId);

	if (DispatchGroupId >= InstanceCullingLoadBalancer_GetNumBatches())
	{
		return;
	}

#if ENABLE_BATCH_MODE
	// Load Instance culling context batch info, indirection per group
	FContextBatchInfo BatchInfo = LoadBatchInfo(DispatchGroupId);
#else // !ENABLE_BATCH_MODE
	// Single Instance culling context batch in the call, set up batch from the kernel parameters
	FContextBatchInfo BatchInfo = (FContextBatchInfo)0;
	BatchInfo.NumViewIds = NumViewIds;
	BatchInfo.DynamicInstanceIdOffset = DynamicInstanceIdOffset;
	BatchInfo.DynamicInstanceIdMax = DynamicInstanceIdMax;
	// Note: for the unbatched case, the permutation will control HZB test, so we set to true
	BatchInfo.bAllowOcclusionCulling = true;
	BatchInfo.bDrawOnlySelected = bDrawOnlySelected;
#endif // ENABLE_BATCH_MODE

	FInstanceCullingSetup InstanceCullingSetup = LoadInstanceCullingSetup(GroupId, GroupThreadIndex, BatchInfo.DynamicInstanceIdOffset, BatchInfo.DynamicInstanceIdMax, GetItemDataOffset(BatchInfo, CurrentBatchProcessingMode));

	FInstanceWorkSetup WorkSetup = InstanceCullingSetup.InstanceWorkSetup;

	if (!WorkSetup.bValid)
	{
		return;
	}

	uint InstanceId = InstanceCullingSetup.InstanceId;

	// Extract the draw command payload
	const FInstanceCullingPayload Payload = LoadInstanceCullingPayload(WorkSetup.Item.Payload, BatchInfo);

	// Load auxiliary per-instanced-draw command info
	const FDrawCommandDesc DrawCommandDesc = UnpackDrawCommandDesc(DrawCommandDescs[Payload.IndirectArgIndex]);

	// Extract compaction data (if applicable)	
#if ENABLE_INSTANCE_COMPACTION
	const bool bCompactInstances = Payload.CompactionDataIndex != 0xFFFFFFFFU;
#else
	const bool bCompactInstances = false;
#endif
	uint CompactOutputInstanceIndex = 0;
	FDrawCommandCompactionData CompactionData = (FDrawCommandCompactionData)0;	
	BRANCH
	if (bCompactInstances)
	{
		CompactionData = UnpackDrawCommandCompactionData(DrawCommandCompactionData[Payload.CompactionDataIndex]);
		
		const uint WorkItemLocalInstanceOffset = WorkSetup.Item.InstanceDataOffset - Payload.InstanceDataOffset;
		CompactOutputInstanceIndex = Payload.RunInstanceOffset + WorkItemLocalInstanceOffset + uint(WorkSetup.LocalItemIndex);
	}

	// TODO: This must be read-modify-written when batching such that the final offset that is fed to the VS is correct.
	//       Then we don't need to add the batch offset (BatchInfo.InstanceDataWriteOffset)
	const uint InstanceDataOutputOffset = InstanceIdOffsetBuffer[Payload.IndirectArgIndex];

	const FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceId);
	const FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

	uint NumVisibleInstances = 0;
#if STEREO_CULLING_MODE
	uint CullingFlags0 = 0;
	uint CullingFlags1 = 0;
	const bool bVisible = IsInstanceVisible(PrimitiveData, InstanceData, InstanceId, BatchInfo.ViewIdsOffset + 0U, BatchInfo.bAllowOcclusionCulling, BatchInfo.bDrawOnlySelected, DrawCommandDesc, CullingFlags0) ||
					      IsInstanceVisible(PrimitiveData, InstanceData, InstanceId, BatchInfo.ViewIdsOffset + 1U, BatchInfo.bAllowOcclusionCulling, BatchInfo.bDrawOnlySelected, DrawCommandDesc, CullingFlags1);
	const uint CullingFlags = CullingFlags0 | CullingFlags1;
	NumVisibleInstances += bVisible ? 2 : 0;
	
	BRANCH
	if (bCompactInstances)
	{
		const uint OutputOffset = CompactOutputInstanceIndex * 2U;
		WriteInstanceForCompaction(CompactionData.SrcInstanceIdOffset + OutputOffset + 0U, bVisible, InstanceId, InstanceData, 0U, CullingFlags);
		WriteInstanceForCompaction(CompactionData.SrcInstanceIdOffset + OutputOffset + 1U, bVisible, InstanceId, InstanceData, 1U, CullingFlags);
	}
	else if (bVisible)
	{
		uint OutputOffset;
		InterlockedAdd(DrawIndirectArgsBufferOut[Payload.IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1], 2U, OutputOffset);
		WriteInstance(InstanceDataOutputOffset + (OutputOffset + 0U) * INSTANCE_DATA_STRIDE_ELEMENTS, InstanceId, PrimitiveData, InstanceData, 0U, CullingFlags, DrawCommandDesc.MeshLODIndex);
		WriteInstance(InstanceDataOutputOffset + (OutputOffset + 1U) * INSTANCE_DATA_STRIDE_ELEMENTS, InstanceId, PrimitiveData, InstanceData, 1U, CullingFlags, DrawCommandDesc.MeshLODIndex);
	}
#else // !STEREO_CULLING_MODE

	for (uint ViewIdIndex = 0; ViewIdIndex < BatchInfo.NumViewIds; ++ViewIdIndex)
	{
		// Culling is disabled for multi-view
		uint CullingFlags = 0;
		bool bVisible = IsInstanceVisible(PrimitiveData, InstanceData, InstanceId, BatchInfo.ViewIdsOffset + ViewIdIndex, BatchInfo.bAllowOcclusionCulling, BatchInfo.bDrawOnlySelected, DrawCommandDesc, CullingFlags);

		NumVisibleInstances += bVisible ? 1 : 0;
		
		BRANCH
		if (bCompactInstances)
		{
			const uint OutputOffset = CompactOutputInstanceIndex * BatchInfo.NumViewIds + ViewIdIndex;
			WriteInstanceForCompaction(CompactionData.SrcInstanceIdOffset + OutputOffset, bVisible, InstanceId, InstanceData, ViewIdIndex, CullingFlags);
		}
		else if (bVisible)
		{
			// TODO: if all items in the group-batch target the same draw args the more efficient warp-collective functions can be used
			//       detected as FInstanceBatch.NumItems == 1. Can switch dynamically or bin the items that fill a group and dispatch separately with permutation.
			// TODO: if the arg only has a single item, and culling is not enabled, then we can skip the atomics. Again do dynamically or separate permutation.
			uint OutputOffset;
			InterlockedAdd(DrawIndirectArgsBufferOut[Payload.IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1], 1U, OutputOffset);
			WriteInstance(InstanceDataOutputOffset + OutputOffset * INSTANCE_DATA_STRIDE_ELEMENTS, InstanceId, PrimitiveData, InstanceData, ViewIdIndex, CullingFlags, DrawCommandDesc.MeshLODIndex);
		}
	}
#endif // STEREO_CULLING_MODE

	BRANCH
	if (bCompactInstances && NumVisibleInstances > 0)
	{
		// Determine compaction block and atomically increment its count
		const uint BlockIndex = GetCompactionBlockIndexFromInstanceIndex(CompactOutputInstanceIndex);
		InterlockedAdd(CompactionBlockCounts[CompactionData.BlockOffset + BlockIndex], NumVisibleInstances);
	}
}


uint NumIndirectArgs;
/**
 */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void ClearIndirectArgInstanceCountCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	uint IndirectArgIndex = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, NUM_THREADS_PER_GROUP);

	if (IndirectArgIndex < NumIndirectArgs)
	{
		DrawIndirectArgsBufferOut[IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1] = 0U;
	}
}
