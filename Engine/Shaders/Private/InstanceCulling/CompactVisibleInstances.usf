// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ComputeShaderUtils.ush"
#include "../ThreadGroupPrefixSum.ush"
#include "InstanceCompactionCommon.ush"

#if COMPILER_SUPPORTS_WAVE_VOTE
	// TODO: Need to also check that wave width == thread group size (or implement workaround for other wave sizes)
	#define COMPACTION_USES_WAVE_OPS 0
#endif

/** Data needed per draw command for compaction */
StructuredBuffer<FPackedDrawCommandCompactionData> DrawCommandCompactionData;

/**
 * CalculateCompactBlockInstanceOffsets
 *
 * Compaction phase 1 - Generate output offsets of "blocks" of source instance IDs such that all valid IDs will be compacted for
 * the draw comand. Each thread group in this dispatch aligns with a single draw command. It uses a hybrid of parallelization and
 * serialization to calculate the compact output offsets in the instance ID buffer in a way that doesn't require multiple dispatches.
 */
#if CALCULATE_COMPACT_BLOCK_INSTANCE_OFFSETS

/** Number of valid instances per block in a draw command (accumulated by BuildInstanceDrawCommands.usf) */
StructuredBuffer<uint> BlockInstanceCounts;

/** Destination instance offset per block of instances in a draw command. (Relative to draw command's InstanceIdOffset) */
RWStructuredBuffer<uint> BlockDestInstanceOffsetsOut;
/** The indirect args buffer to receive the final, compacted instance count */
RWBuffer<uint> DrawIndirectArgsBufferOut;

/** Entry point */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void CalculateCompactBlockInstanceOffsetsCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint DrawCommandIndex = GetUnWrappedDispatchGroupId(GroupId);
	const FDrawCommandCompactionData DrawCommandData = UnpackDrawCommandCompactionData(DrawCommandCompactionData[DrawCommandIndex]);
	const uint BlockOffset = DrawCommandData.BlockOffset;
	const uint NumBlocksInDrawCommand = GetCompactionBlockCount(DrawCommandData.NumInstances);
	const uint NumWorkIterations = ((NumBlocksInDrawCommand - 1U) / NUM_THREADS_PER_GROUP) + 1U;

	uint CurInstanceOffset = 0;
	uint CurDrawCommandBlockIndex = 0;
	for (uint IterIndex = 0; IterIndex < NumWorkIterations; ++IterIndex)
	{
		const uint DrawCommandBlockIndex = CurDrawCommandBlockIndex + GroupThreadIndex;
		const uint GlobalBlockIndex = DrawCommandData.BlockOffset + DrawCommandBlockIndex;

		uint BlockInstanceDestCount = 0;
		BRANCH
		if (DrawCommandBlockIndex < NumBlocksInDrawCommand)
		{
			BlockInstanceDestCount = BlockInstanceCounts[GlobalBlockIndex];
		}

		uint PrefixSum, GroupSum;
	#if COMPACTION_USES_WAVE_OPS
		PrefixSum = WavePrefixSum(BlockInstanceDestCount);
		GroupSum = WaveActiveSum(BlockInstanceDestCount);
	#else
		PrefixSum = ThreadGroupPrefixSum(BlockInstanceDestCount, GroupThreadIndex, GroupSum);
	#endif // COMPACTION_USES_WAVE_OPS
		
		// Write out the instance offsets
		BRANCH
		if (DrawCommandBlockIndex < NumBlocksInDrawCommand)
		{
			BlockDestInstanceOffsetsOut[GlobalBlockIndex] = CurInstanceOffset + PrefixSum;
		}

		CurInstanceOffset += GroupSum;
		CurDrawCommandBlockIndex += NUM_THREADS_PER_GROUP;
	}

	if (GroupThreadIndex == 0)
	{
		// Set the final indirect arg instance count
		DrawIndirectArgsBufferOut[DrawCommandData.IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1] = CurInstanceOffset;
	}
}

#endif // CALCULATE_COMPACT_BLOCK_INSTANCE_OFFSETS

/**
 * CompactVisibleInstances
 *
 * Compaction phase 2 - Write out the visible instances in compact order
 * Each group outputs a compact block of visible instances. The max source (uncompacted) block size is equal to
 * COMPACTION_BLOCK_NUM_INSTANCES * NumViews. The number of threads in the group is equal to the number of instances
 * per block.
 */ 
#if COMPACT_VISIBLE_INSTANCES

/** Draw command index per block */
StructuredBuffer<uint> BlockDrawCommandIndices;
/** The raw, uncompacted instance ID buffer (provided by BuildInstanceDrawCommands.usf) */
StructuredBuffer<uint> InstanceIdsBufferIn;
/** Instance output offsets per block (provided by CalculateCompactBlockInstanceOffsetsCS) */
StructuredBuffer<uint> BlockDestInstanceOffsets;

/** The compacted output buffer of InstanceIds (non-mobile) */
RWStructuredBuffer<uint> InstanceIdsBufferOut;
/** The compacted output buffer of instance data consumed by vertex shaders on mobile */
RWByteAddressBuffer InstanceIdsBufferOutMobile;

/** Entry point */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void CompactVisibleInstances(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint GlobalBlockIndex = GetUnWrappedDispatchGroupId(GroupId);
	const FDrawCommandCompactionData DrawCommandData = UnpackDrawCommandCompactionData(DrawCommandCompactionData[BlockDrawCommandIndices[GlobalBlockIndex]]);
	const uint DrawCommandBlockIndex = GlobalBlockIndex - DrawCommandData.BlockOffset;
	const uint MaxInstancesPerBlock = NUM_THREADS_PER_GROUP * DrawCommandData.NumViews;
	const uint BlockSrcInstanceCount = min(DrawCommandData.NumInstances * DrawCommandData.NumViews - DrawCommandBlockIndex * MaxInstancesPerBlock, MaxInstancesPerBlock);
	const uint BlockSrcInstanceOffset = DrawCommandData.SrcInstanceIdOffset + DrawCommandBlockIndex * MaxInstancesPerBlock;
	const uint BlockDestInstanceOffset = DrawCommandData.DestInstanceIdOffset + BlockDestInstanceOffsets[GlobalBlockIndex];

	// These are the relative offsets for the whole group from the start of the source and destination offsets for the block, respectively
	uint CurGroupSrcOffset = 0;
	uint CurGroupDestOffset = 0;

	while (CurGroupSrcOffset < BlockSrcInstanceCount) // may loop as many times as NumViews
	{
		uint InstanceId = 0xFFFFFFFFU;
		
		const uint BlockSrcInstanceIndex = CurGroupSrcOffset + GroupThreadIndex;
		const uint GlobalSrcInstanceIndex = BlockSrcInstanceOffset + BlockSrcInstanceIndex;
		BRANCH
		if (BlockSrcInstanceIndex < BlockSrcInstanceCount)
		{
			InstanceId = InstanceIdsBufferIn[GlobalSrcInstanceIndex];
		}
		
		const bool bIsValidInstance = (InstanceId != 0xFFFFFFFFU);

		// now calculate the index based on the valid index in the group
		uint PrefixSum, GroupSum;
	#if COMPACTION_USES_WAVE_OPS
		PrefixSum = WavePrefixCountBits(bIsValidInstance);
		GroupSum = WaveActiveCountBits(bIsValidInstance);
	#else
		// NOTE: This must be called even by threads that won't output an InstanceId
		PrefixSum = ThreadGroupPrefixSum(bIsValidInstance ? 1U : 0U, GroupThreadIndex, GroupSum);
	#endif // COMPACTION_USES_WAVE_OPS

		if (bIsValidInstance)
		{
			const uint BlockDestInstanceIndex = CurGroupDestOffset + PrefixSum;
			const uint GlobalDestInstanceIndex = BlockDestInstanceOffset + BlockDestInstanceIndex;

			// TODO: This currently doesn't support mobile, which outputs transform data into a vertex buffer
			InstanceIdsBufferOut[GlobalDestInstanceIndex] = InstanceId;
		}

		CurGroupSrcOffset += NUM_THREADS_PER_GROUP;
		CurGroupDestOffset += GroupSum;
	}	
}

#endif // COMPACT_VISIBLE_INSTANCES