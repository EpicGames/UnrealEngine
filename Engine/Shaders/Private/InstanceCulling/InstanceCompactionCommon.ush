// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef COMPACTION_BLOCK_NUM_INSTANCES
	#error COMPACTION_BLOCK_NUM_INSTANCES must be defined
#endif

/** Determines number of compaction blocks required for the specified number of instances */
uint GetCompactionBlockCount(uint NumInstances)
{
	return ((NumInstances - 1U) / COMPACTION_BLOCK_NUM_INSTANCES) + 1U;
}

/** Determines number of compaction blocks required for the specified number of instances */
uint GetCompactionBlockIndexFromInstanceIndex(uint InstanceIndex)
{
	return InstanceIndex / COMPACTION_BLOCK_NUM_INSTANCES;
}

struct FPackedDrawCommandCompactionData
{
	uint NumInstances_NumViews;
	uint BlockOffset;
	uint IndirectArgIndex;
	uint SrcInstanceIdOffset;
	uint DestInstanceIdOffset;
};

struct FDrawCommandCompactionData
{
	/** The number of views for which the instances were written */
	uint NumViews;
	/** The total number of instances in the draw command */
	uint NumInstances;
	/** The offset of the first compaction "block" (group of N instances * NumViewIds) of the draw command */
	uint BlockOffset;
	/** The indirect argument index of the draw command */
	uint IndirectArgIndex;
	/** The offset of the input instance ID buffer for the start of the draw command before compaction */
	uint SrcInstanceIdOffset;
	/** The offset of the output instance ID buffer for the start of the draw command after compaction */
	uint DestInstanceIdOffset;
};

FDrawCommandCompactionData UnpackDrawCommandCompactionData(FPackedDrawCommandCompactionData Data)
{
	FDrawCommandCompactionData Output;
	Output.NumViews = Data.NumInstances_NumViews & 0xFFU;
	Output.NumInstances = Data.NumInstances_NumViews >> 8;
	Output.BlockOffset = Data.BlockOffset;
	Output.IndirectArgIndex = Data.IndirectArgIndex;
	Output.SrcInstanceIdOffset = Data.SrcInstanceIdOffset;
	Output.DestInstanceIdOffset = Data.DestInstanceIdOffset;

	return Output;
}