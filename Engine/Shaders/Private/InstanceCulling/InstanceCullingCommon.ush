// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "/Engine/Shared/InstanceCullingDefinitions.h"

#ifndef BATCH_PROCESSING_MODE_NUM
#define BATCH_PROCESSING_MODE_NUM 1
#endif // BATCH_PROCESSING_MODE_NUM

#ifndef ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS
#define ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS 0
#endif // ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS

struct FDrawCommandDesc
{
	bool bMaterialUsesWorldPositionOffset;
	bool bMaterialAlwaysEvaluatesWorldPositionOffset;
	uint MeshLODIndex;
	float MinScreenSize;
	float MaxScreenSize;
	int DynamicMeshBoundsIndex;
};

FDrawCommandDesc UnpackDrawCommandDesc(uint2 PackedData)
{
	FDrawCommandDesc Desc;
	Desc.bMaterialUsesWorldPositionOffset = (PackedData.x & 1U) != 0;
	Desc.bMaterialAlwaysEvaluatesWorldPositionOffset = ((PackedData.x >> 1U) & 1U) != 0;
	Desc.MeshLODIndex = (PackedData.x >> 2U) & 0xFU;
	Desc.MinScreenSize = (float)((PackedData.x >> 6U) & 0x1FFFU) / (float)(1 << 10);
	Desc.MaxScreenSize = (float)((PackedData.x >> 19U) & 0x1FFFU) / (float)(1 << 10);
	Desc.DynamicMeshBoundsIndex = int(PackedData.y);
	return Desc;
}

struct FContextBatchInfoPacked
{
	uint IndirectArgOffset;
	uint InstanceDataWriteOffset;
	uint PayloadOffset;
	uint CompactionDataOffset;
	uint ViewIdsOffset;
	uint NumViewIds_Flags;
	uint DynamicInstanceIdOffset;
	uint DynamicInstanceIdMax;
	// Must be equal to ItemDataOffset[BATCH_PROCESSING_MODE_NUM]
	// Not using array here to avoid GL shader compiler bug on Adreno devices
	uint ItemDataOffset0;
	uint ItemDataOffset1;
};

struct FContextBatchInfo
{
	uint IndirectArgOffset;
	uint InstanceDataWriteOffset;
	uint PayloadOffset;
	uint CompactionDataOffset;
	uint ViewIdsOffset;
	uint NumViewIds;
	bool bAllowOcclusionCulling;
	bool bDrawOnlySelected;
	uint DynamicInstanceIdOffset;
	uint DynamicInstanceIdMax;
	// Must be equal to ItemDataOffset[BATCH_PROCESSING_MODE_NUM]
	// Not using array here to avoid GL shader compiler bug on Adreno devices
	uint ItemDataOffset0;
	uint ItemDataOffset1;
};

uint GetItemDataOffset(FContextBatchInfo BatchInfo, uint BatchProcessingMode)
{
#if BATCH_PROCESSING_MODE_NUM > 2
#error Make sure FContextBatchInfo::ItemDataOffset is up to date with BATCH_PROCESSING_MODE_NUM
#else
	return (BatchProcessingMode == 0U) ? BatchInfo.ItemDataOffset0 : BatchInfo.ItemDataOffset1;
#endif
}

#if ENABLE_BATCH_MODE
StructuredBuffer<FContextBatchInfoPacked> BatchInfos;
StructuredBuffer<uint> BatchInds;
FContextBatchInfo LoadBatchInfo(uint GroupIndex)
{
	FContextBatchInfoPacked ContextBatchInfoPacked = BatchInfos[BatchInds[GroupIndex]];
	FContextBatchInfo ContextBatchInfo;
	ContextBatchInfo.IndirectArgOffset = ContextBatchInfoPacked.IndirectArgOffset;
	ContextBatchInfo.InstanceDataWriteOffset = ContextBatchInfoPacked.InstanceDataWriteOffset;
	ContextBatchInfo.PayloadOffset = ContextBatchInfoPacked.PayloadOffset;
	ContextBatchInfo.CompactionDataOffset = ContextBatchInfoPacked.CompactionDataOffset;
	ContextBatchInfo.ViewIdsOffset = ContextBatchInfoPacked.ViewIdsOffset;
	ContextBatchInfo.NumViewIds = ContextBatchInfoPacked.NumViewIds_Flags >> INSTANCE_CULLING_FLAGS_BITS;
	ContextBatchInfo.bAllowOcclusionCulling = (ContextBatchInfoPacked.NumViewIds_Flags & INSTANCE_CULLING_FLAGS_ALLOW_OCCLUSION_CULLING) != 0u;
	ContextBatchInfo.bDrawOnlySelected		= (ContextBatchInfoPacked.NumViewIds_Flags & INSTANCE_CULLING_FLAGS_DRAW_ONLY_SELECTED) != 0u;
	ContextBatchInfo.DynamicInstanceIdOffset = ContextBatchInfoPacked.DynamicInstanceIdOffset;
	ContextBatchInfo.DynamicInstanceIdMax = ContextBatchInfoPacked.DynamicInstanceIdMax;
	ContextBatchInfo.ItemDataOffset0 = ContextBatchInfoPacked.ItemDataOffset0;
	ContextBatchInfo.ItemDataOffset1 = ContextBatchInfoPacked.ItemDataOffset1;
	return ContextBatchInfo;
}
#endif // ENABLE_BATCH_MODE

struct FInstanceCullingPayload
{
	/** Whether or not the instances are dynamic */
	bool bDynamicInstanceDataOffset;
	/** The IndirectArgs index for instances in the run. */
	uint IndirectArgIndex;
	/** The instance data offset of the start of the current instance run, or -1 if this data should not be needed for this work item */
	uint InstanceDataOffset;
	/** The number of instances added from other runs before this one, or -1 if this data should not be needed for this work item */
	uint RunInstanceOffset;
	/** When compaction is enabled, the index into the FDrawCommandCompactionData buffer (or -1 otherwise) */
	uint CompactionDataIndex;
};

struct FPackedInstanceCullingPayload
{
	uint bDynamicInstanceDataOffset_IndirectArgIndex;
	uint InstanceDataOffset;
	uint RunInstanceOffset;
	uint CompactionDataIndex;
};

bool IsExtendedInstanceCullingPayload(uint WorkItemPayload)
{
	return (WorkItemPayload & INSTANCE_CULLING_PRESERVE_INSTANCE_ORDER_BIT_MASK) != 0U;
}

bool IsDynamicInstanceDataOffset(uint WorkItemPayload)
{
	return (WorkItemPayload & INSTANCE_CULLING_DYNAMIC_INSTANCE_DATA_OFFSET_BIT_MASK) != 0U;
}

FInstanceCullingPayload UnpackInstanceCullingPayload(FPackedInstanceCullingPayload Payload)
{
	FInstanceCullingPayload Output;
	Output.bDynamicInstanceDataOffset = (Payload.bDynamicInstanceDataOffset_IndirectArgIndex >> 31U) != 0U;
	Output.IndirectArgIndex = Payload.bDynamicInstanceDataOffset_IndirectArgIndex & 0x7FFFFFFFU;
	Output.InstanceDataOffset = Payload.InstanceDataOffset;
	Output.RunInstanceOffset = Payload.RunInstanceOffset;
	Output.CompactionDataIndex = Payload.CompactionDataIndex;

	return Output;
}

#if ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS
StructuredBuffer<FPackedInstanceCullingPayload> InstanceCullingPayloads;
#endif

/**
 * Retrieves the instance culling payload data. When instance compaction is not needed to preserve instance order, much of the
 * instance culling payload need not take up space, so it is compacted into the 26-bit payload provided by the load balancer.
 * Otherwise, the first bit of the load balancer's payload is set, and the remaining 25 bits are used to index into the payload
 * structured buffer.
 */
FInstanceCullingPayload LoadInstanceCullingPayload(uint WorkItemPayload, FContextBatchInfo BatchInfo)
{
	FInstanceCullingPayload Output;

	Output.bDynamicInstanceDataOffset = IsDynamicInstanceDataOffset(WorkItemPayload);

#if ENABLE_EXTENDED_INSTANCE_CULLING_PAYLOADS
	BRANCH
	if (IsExtendedInstanceCullingPayload(WorkItemPayload))
	{
		Output = UnpackInstanceCullingPayload(InstanceCullingPayloads[BatchInfo.PayloadOffset + (WorkItemPayload >> INSTANCE_CULLING_PAYLOAD_NUM_COMMON_BITS)]);
		Output.IndirectArgIndex += BatchInfo.IndirectArgOffset;
		Output.CompactionDataIndex += BatchInfo.CompactionDataOffset;
		return Output;
	}
#else
	// Ensure that the payload is not extended
	checkSlow(!IsExtendedInstanceCullingPayload(WorkItemPayload));
#endif

	Output.IndirectArgIndex = BatchInfo.IndirectArgOffset + (WorkItemPayload >> INSTANCE_CULLING_PAYLOAD_NUM_COMMON_BITS);
	Output.InstanceDataOffset = 0xFFFFFFFFU;
	Output.RunInstanceOffset = 0xFFFFFFFFU;
	Output.CompactionDataIndex = 0xFFFFFFFFU;

	return Output;
}