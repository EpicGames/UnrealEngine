// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "../ComputeShaderUtils.ush"

#if COMPILER_SUPPORTS_WAVE_VOTE
	// TODO: Need to also check that wave width == thread group size (or implement workaround for other wave sizes)
#define USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION 0
#endif

// Whether to use a special case where all items are expected to have a single instance
#ifndef LOAD_BALANCER_SINGLE_INSTANCE_MODE
	#define LOAD_BALANCER_SINGLE_INSTANCE_MODE 0
#endif

struct FPackedInstanceBatch
{
	uint FirstItem_NumItems;
};

struct FPackedInstanceBatchItem
{
	// packed 32-NumInstancesItemBits:NumInstancesItemBits - need one more bit for the case where one item has ThreadGroupSize work to do
	uint InstanceDataOffset_NumInstances;
	// packed 32-PrefixBits:PrefixBits
	uint Payload_BatchPrefixOffset;
};


StructuredBuffer<FPackedInstanceBatch> BatchBuffer;
StructuredBuffer<FPackedInstanceBatchItem> ItemBuffer;
uint NumBatches;
uint NumItems;
uint NumGroupsPerBatch;


struct FInstanceBatch
{
	uint FirstItem;
	uint NumItems;
};

FInstanceBatch UnpackBatch(FPackedInstanceBatch PackedBatch, uint ItemDataOffset)
{
	FInstanceBatch Result;
	Result.FirstItem = ItemDataOffset + (PackedBatch.FirstItem_NumItems >> NUM_INSTANCES_ITEM_BITS);
	Result.NumItems = PackedBatch.FirstItem_NumItems & NUM_INSTANCES_ITEM_MASK;

	return Result;
}

struct FInstanceBatchItem
{
	uint InstanceDataOffset;
	uint NumInstances;
	uint Payload;
	uint BatchPrefixOffset;
};

FInstanceBatchItem UnpackItem(FPackedInstanceBatchItem PackedItem)
{
	FInstanceBatchItem Result;

	Result.InstanceDataOffset = PackedItem.InstanceDataOffset_NumInstances >> NUM_INSTANCES_ITEM_BITS;
	Result.NumInstances = PackedItem.InstanceDataOffset_NumInstances & NUM_INSTANCES_ITEM_MASK;
	Result.Payload = PackedItem.Payload_BatchPrefixOffset >> PREFIX_BITS;
	Result.BatchPrefixOffset = PackedItem.Payload_BatchPrefixOffset & PREFIX_BIT_MASK;

	return Result;
}


#if USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION

groupshared uint WorkBoundary[NUM_THREADS_PER_GROUP];

#else // !USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION

groupshared uint ItemIndex[NUM_THREADS_PER_GROUP];

void PrefixMaxStep(uint ElementIndex, uint StepSize, inout uint CurrentValue)
{
	if (ElementIndex >= StepSize)
	{
		CurrentValue = max(CurrentValue, ItemIndex[ElementIndex - StepSize]);
	}
	GroupMemoryBarrierWithGroupSync();

	if (ElementIndex >= StepSize)
	{
		ItemIndex[ElementIndex] = CurrentValue;
	}
	GroupMemoryBarrierWithGroupSync();
}

uint PrefixMaxLastStep(uint ElementIndex, inout uint CurrentValue)
{
	GroupMemoryBarrierWithGroupSync();

	uint StepSize = NUM_THREADS_PER_GROUP / 2U;

	if (ElementIndex >= StepSize)
	{
		uint NewValue = max(CurrentValue, ItemIndex[ElementIndex - StepSize]);
		CurrentValue = NewValue;
	}

	// No sync needed since we don't look at ItemIndex[] again
	return CurrentValue;
}

uint PrefixMax(uint ElementIndex)
{
	GroupMemoryBarrierWithGroupSync();

	uint CurrentValue = ItemIndex[ElementIndex];
	GroupMemoryBarrierWithGroupSync();

	// Note: manually unrolled as UNROLL seems to not figure out *= 2, at least on some platforms
	//for (uint StepSize = 1U; StepSize < NUM_THREADS_PER_GROUP / 2U; StepSize *= 2u)
	//{
	//	PrefixMaxStep(ElementIndex, StepSize, CurrentValue);
	//}
	PrefixMaxStep(ElementIndex, 1U, CurrentValue);
	PrefixMaxStep(ElementIndex, 2U, CurrentValue);
	PrefixMaxStep(ElementIndex, 4U, CurrentValue);
	PrefixMaxStep(ElementIndex, 8U, CurrentValue);
#if NUM_THREADS_PER_GROUP > 32u
	PrefixMaxStep(ElementIndex, 16U, CurrentValue);
#endif
#if NUM_THREADS_PER_GROUP > 64u
	PrefixMaxStep(ElementIndex, 32U, CurrentValue);
#endif
#if NUM_THREADS_PER_GROUP > 128u
	PrefixMaxStep(ElementIndex, 64U, CurrentValue);
#endif
#if NUM_THREADS_PER_GROUP > 256u
	PrefixMaxStep(ElementIndex, 128U, CurrentValue);
#endif

	return PrefixMaxLastStep(ElementIndex, CurrentValue);
}

#endif // USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION


groupshared FInstanceBatchItem Items[NUM_THREADS_PER_GROUP];

struct FInstanceWorkSetup
{
	bool bValid;
	FInstanceBatchItem Item;
	// Work out who I am in expansion of  the item in range [0,WorkItem.Count], may be negative when no work at end of batch
	int LocalItemIndex;
	uint WorkItemIndex;
	uint GroupIndexPerBatch;		// 0..NumGroupsPerBatch if multiple groups per batch are requested
};


// Set bSupportMultipleBatches to true if this kernel is dispatched with more than 1 group per batch.
FInstanceWorkSetup InstanceCullingLoadBalancer_Setup(uint3 GroupId, uint GroupThreadIndex, uint ItemDataOffset, bool bSupportMultipleBatches = false)
{
	FInstanceWorkSetup  Setup = (FInstanceWorkSetup)0;

	uint BatchIndex = GetUnWrappedDispatchGroupId(GroupId);
	// Interleave multiple groups per batch if multiple groups per batch are requested
	if (bSupportMultipleBatches)
	{
		Setup.GroupIndexPerBatch = BatchIndex % NumGroupsPerBatch;
		BatchIndex = BatchIndex / NumGroupsPerBatch;
	}

	if (BatchIndex >= NumBatches)
	{
		return Setup;
	}

	// Load work description for whole work group
	FInstanceBatch GroupBatch = UnpackBatch(BatchBuffer[BatchIndex], ItemDataOffset);

#if LOAD_BALANCER_SINGLE_INSTANCE_MODE
	// Special case where all items are expected to have a single instance
	if (GroupBatch.NumItems > GroupThreadIndex)
	{
		Setup.Item = UnpackItem(ItemBuffer[GroupBatch.FirstItem + GroupThreadIndex]);
		Setup.bValid = true;
	}
#else
	// Special case where we have an 1:1 item/thread no need to do prefix sums or whatever
	// TODO: this case can also store less data per item
	if (GroupBatch.NumItems == NUM_THREADS_PER_GROUP)
	{
		checkSlow(UnpackItem(ItemBuffer[GroupBatch.FirstItem + GroupThreadIndex]).BatchPrefixOffset == GroupThreadIndex);

		Setup.Item = UnpackItem(ItemBuffer[GroupBatch.FirstItem + GroupThreadIndex]);
		Setup.bValid = true;

		return Setup;
	}

	// Load work Items (guaranteed to be less than group size)
#if USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION
	WorkBoundary[GroupThreadIndex] = 0;
#else 
	ItemIndex[GroupThreadIndex] = 0U;
#endif

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadIndex < GroupBatch.NumItems)
	{
		FInstanceBatchItem WorkItem = UnpackItem(ItemBuffer[GroupBatch.FirstItem + GroupThreadIndex]);
#if USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION
		// Mark last instance as boundary
		WorkBoundary[(WorkItem.BatchPrefixOffset + WorkItem.NumInstances) - 1U] = 1;
#else 
		ItemIndex[WorkItem.BatchPrefixOffset] = GroupThreadIndex;
#endif
		Items[GroupThreadIndex] = WorkItem;
	}
	GroupMemoryBarrierWithGroupSync();
	uint WorkItemIndex = 0U;
	if (GroupBatch.NumItems < 4U)
	{
		// Distribute using linear search (no syncs requires like the prefix max op)
		for (int Index = int(GroupBatch.NumItems) - 1; Index >= 0; --Index)
		{
			// TODO: pack the offset data into a single dword and store in group batch? 6x5 bits can fit and would be far more efficient that all the shared mumbojumbo
			FInstanceBatchItem WorkItem = Items[Index];
			if (GroupThreadIndex >= WorkItem.BatchPrefixOffset)
			{
				WorkItemIndex = uint(Index);
				break;
			}
		}
	}
	else
	{
#if USE_WAVE_BIT_PREFIX_WORK_DISTRIBUTION
		// Ballot based boundary search for when wave ops are available.
		bool bIsBoundary = WorkBoundary[GroupThreadIndex];
		WorkItemIndex = WavePrefixCountBits(bIsBoundary);
#else
		// Distribute the index to the other threads that should work on the same item
		WorkItemIndex = PrefixMax(GroupThreadIndex);
#endif
	}

	Setup.WorkItemIndex = WorkItemIndex;
	Setup.Item = Items[WorkItemIndex];
	// Work out who I am in expansion of  the item in range [0,WorkItem.Count], may be negative when no work at end of batch
	Setup.LocalItemIndex = int(GroupThreadIndex - Setup.Item.BatchPrefixOffset);
	Setup.bValid = Setup.LocalItemIndex >= 0 && Setup.LocalItemIndex < int(Setup.Item.NumInstances);
#endif //LOAD_BALANCER_SINGLE_INSTANCE_MODE

	return Setup;
}


uint InstanceCullingLoadBalancer_GetNumBatches()
{
	return NumBatches;
}