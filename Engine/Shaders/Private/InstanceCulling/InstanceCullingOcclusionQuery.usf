// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneData.ush"
#include "../VertexFactoryCommon.ush" // for TransformLocalToTranslatedWorld
#include "../Nanite/NaniteHZBCull.ush"

// We need to define NUM_THREADS_PER_GROUP before including InstanceCullingSetup as it includes InstanceCullingLoadBalancer.ush which requires declares groupshared uint ItemIndex[NUM_THREADS_PER_GROUP];
#ifndef NUM_THREADS_PER_GROUP_DEFAULT
#	define NUM_THREADS_PER_GROUP_DEFAULT 64
#endif // NUM_THREADS_PER_GROUP_DEFAULT

#ifndef NUM_THREADS_PER_GROUP
#	define NUM_THREADS_PER_GROUP NUM_THREADS_PER_GROUP_DEFAULT
#endif // NUM_THREADS_PER_GROUP

// InstanceCullingSetup.ush pulls some variables that get set by FInstanceProcessingGPULoadBalancer::SetShaderDefines. We only call that when USE_LOAD_BALANCER == 1
#if USE_LOAD_BALANCER 
#define LOAD_BALANCER_SINGLE_INSTANCE_MODE 0
#include "InstanceCullingSetup.ush"
#endif

#ifndef HZB_DESIRED_FOOTPRINT_PIXELS
#	define HZB_DESIRED_FOOTPRINT_PIXELS 4
#endif

#if !PLATFORM_SUPPORTS_VERTEX_SHADER_SRVS
#error This shader requires accessing GPUScene buffers as SRVs from vertex stage
#endif

#ifndef DIM_MULTI_VIEW
#	define DIM_MULTI_VIEW 0
#endif

RWBuffer<uint> RWVisibilityMask;

uint NumInstances;
uint ViewMask;
float OcclusionSlop;

#define EInstanceVisibilityStatus uint // Not all platforms support native enums
#define IVS_Hidden          0 // Instance is hidden or invalid, no occlusion query needed
#define IVS_Visible         1 // Instance is definitely visible, no occlusion query needed
#define IVS_PossiblyVisible 2 // Occlusion query is needed to determine visibility
#define IVS_Incompatible    3 // Instance can't use occlusion queries

EInstanceVisibilityStatus GetInstanceDataAndVisibility(uint InstanceId, out FInstanceSceneData OutInstanceData)
{
	const FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceId);

	OutInstanceData = InstanceData;

	if (!InstanceData.ValidInstance)
	{
		return IVS_Hidden;
	}

	const FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

	if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_CULLING_OCCLUSION_QUERIES) == 0u)
	{
		// If instance culling occlusion queries are not allowed for this primitive, it is marked as "visible"
		return IVS_Incompatible;
	}

	float3 BoxOrigin = InstanceData.LocalBoundsCenter;
	float3 BoxExtent = InstanceData.LocalBoundsExtent + OcclusionSlop;

	const bool bIsOrtho = IsOrthoProjection(ResolvedView.ViewToClip);

	float4x4 LocalToTranslatedWorld = DFMultiplyAffineTranslationDemote(InstanceData.LocalToWorld, ResolvedView.PreViewTranslation);

	FFrustumCullData Cull = BoxCullFrustum(
		BoxOrigin.xyz,
		BoxExtent.xyz,
		LocalToTranslatedWorld,
		ResolvedView.TranslatedWorldToClip,
		ResolvedView.ViewToClip,
		bIsOrtho, true /* near clip */, false /* skip culling */
	);

	if (Cull.bCrossesNearPlane)
	{
		// Occlusion query can only be used when the proxy box is fully in front of the camera.
		return IVS_Visible;
	}

	if (!Cull.bIsVisible)
	{
		// Conservative culling heuristic:
		// If instance is frustum-culled this frame, it may become visible next frame when instance visibility buffer is consumed.
		// Marking the instance as visible allows InstanceCullBuildInstanceIdBufferCS to perform a more accurate check next frame.
		// NOTE: There is still a possibility of instance getting occlusion-culled this frame while it should be visible.
		//       Two-pass occlusion culling algorithm is the only robust solution.
		return IVS_Visible;
	}

	int4 HZBViewRect = int4(0, 0, HZBViewSize.x, HZBViewSize.y);
	FScreenRect Rect = GetScreenRect(HZBViewRect, Cull, HZB_DESIRED_FOOTPRINT_PIXELS);

	if (any(Rect.Pixels == HZBViewRect))
	{
		// Conservative culling heuristic:
		// Objects may be visible with HZB test but occluded with per-pixel depth test during this frame.
		// However in the next frame the object may be completely visible and will cause a visible pop-in.
		// Taking the more conservative HZB visibility test result for objects at the edges of the screen reduces the artifacts.
		// This pop-in is still *possible* with HZB test, it is simply slightly less likely.
		// NOTE: There is still a possibility of instance getting occlusion-culled this frame while it should be visible.
		//       Two-pass occlusion culling algorithm is the only robust solution.
		return IVS_Visible;
	}

	if (IsVisibleHZB(Rect, true /*bSample4x4*/))
	{
		// Instance is visible using conservative HZB test and may benefit from a more accurate per-pixel occlusion test.
		return IVS_PossiblyVisible;
	}
	else
	{
		// Instance is definitely hidden and we can skip per-pixel test as it has non-trivial cost.
		return IVS_Hidden;
	}
}

// TODO: could also add a wave-op variant in the future if there is evidence of significant perf overhead
groupshared uint GroupNumVisibleInstances;
groupshared uint GroupVisibleInstances[NUM_THREADS_PER_GROUP];
groupshared uint GroupOutputBaseIndex;

RWBuffer<uint> OutIndirectArgsBuffer;
RWBuffer<uint> OutInstanceIdBuffer;

// Remap linear visible index to a visible GPUScene instance index
Buffer<uint> InstanceIdBuffer;

bool LoadInstanceId(uint3 GroupId, uint GroupThreadIndex, uint DispatchThreadId, inout uint InstanceId)
{
#if USE_LOAD_BALANCER
    // No need to properly fetch the final instance id for dynamic primitives since we're going to skip them anyway
    uint DynamicInstanceIdOffset = 0;
    uint DynamicInstanceIdMax = 0;
	FInstanceCullingSetup InstanceCullingSetup = LoadInstanceCullingSetup(GroupId, GroupThreadIndex, DynamicInstanceIdOffset, DynamicInstanceIdMax, 0);
	InstanceId = InstanceCullingSetup.InstanceId;
    return InstanceCullingSetup.bValid && !InstanceCullingSetup.bIsDynamic;
#else
	const bool bValidInvocation = DispatchThreadId < NumInstances;
	uint ClampedThreadId = min(DispatchThreadId, NumInstances - 1);
	InstanceId = InstanceIdBuffer[ClampedThreadId];
    return bValidInvocation;
#endif
}

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void MainCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex, uint DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	if (GroupThreadIndex == 0)
	{
		GroupNumVisibleInstances = 0;
	}

    uint InstanceId = 0;
	const bool bValidInvocation = LoadInstanceId(GroupId, GroupThreadIndex, DispatchThreadId, InstanceId);

	GroupMemoryBarrierWithGroupSync();

	if (DispatchThreadId == 0)
	{
		// First thread fills the indirect draw args
		// Fill FRHIDrawIndexedIndirectParameters
		OutIndirectArgsBuffer[0] = 36; // IndexCountPerInstance -- 12 triangles per cube
		// OutIndirectArgsBuffer[1] = 0;  // InstanceCount (filled previously by ClearUAV as we're atomically adding to it)
		OutIndirectArgsBuffer[2] = 0; // StartIndexLocation
		OutIndirectArgsBuffer[3] = 0; // BaseVertexLocation
		OutIndirectArgsBuffer[4] = 0; // StartInstanceLocation
	}

	FInstanceSceneData InstanceData = (FInstanceSceneData)0;
	EInstanceVisibilityStatus Status = IVS_Hidden;

	if (bValidInvocation)
	{
		Status = GetInstanceDataAndVisibility(InstanceId, /*out*/ InstanceData);

#if DIM_MULTI_VIEW
		const uint OldVisibilityMask = RWVisibilityMask[InstanceId];
		const uint MaskValueHidden = OldVisibilityMask & (~ViewMask);
		const uint MaskValueVisible = OldVisibilityMask | ViewMask;
#else // DIM_MULTI_VIEW
		const uint MaskValueHidden = 0;
		const uint MaskValueVisible = ViewMask;
#endif // DIM_MULTI_VIEW

		if (Status == IVS_Visible || Status == IVS_Incompatible)
		{
			RWVisibilityMask[InstanceId] = MaskValueVisible; // this is the final value
		}
		else if (Status == IVS_Hidden)
		{
			RWVisibilityMask[InstanceId] = MaskValueHidden; // this is the final value
		}
		else // Status == IVS_PossiblyVisible
		{
			RWVisibilityMask[InstanceId] = MaskValueHidden; // clear the value which will be updated by pixel shader later

			uint LocalOutputIndex = 0;
			InterlockedAdd(GroupNumVisibleInstances, 1, LocalOutputIndex);
			GroupVisibleInstances[LocalOutputIndex] = InstanceId;
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadIndex == 0)
	{
		// One thread per group atomically adds to InstanceCount
		InterlockedAdd(OutIndirectArgsBuffer[1], GroupNumVisibleInstances, GroupOutputBaseIndex);
	}

	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadIndex < GroupNumVisibleInstances)
	{
		OutInstanceIdBuffer[GroupOutputBaseIndex + GroupThreadIndex] = GroupVisibleInstances[GroupThreadIndex];
	}
}

void MainVS(
	float3 InPosition : ATTRIBUTE0,
	uint InstanceId : SV_InstanceID,
	out nointerpolation uint OutInstanceId : INSTANCE_ID,
	out nointerpolation uint OutVisibilityMask : VISIBILITY_MASK,
	out float4 OutPosition : SV_POSITION
)
{
	ResolvedView = ResolveView();

	FInstanceSceneData InstanceData = (FInstanceSceneData)0;

	InstanceId = InstanceIdBuffer[InstanceId];
	GetInstanceDataAndVisibility(InstanceId, /*out*/ InstanceData);

	float3 LocalPosition = (InPosition * (InstanceData.LocalBoundsExtent + OcclusionSlop) * InstanceData.DeterminantSign) + InstanceData.LocalBoundsCenter;
	float4 WorldPosition = TransformLocalToTranslatedWorld(LocalPosition, InstanceData.LocalToWorld);
	OutPosition = mul(WorldPosition, ResolvedView.TranslatedWorldToClip);

	OutInstanceId = InstanceId;

#if DIM_MULTI_VIEW
	const uint OldVisibilityMask = RWVisibilityMask[InstanceId];
	OutVisibilityMask = OldVisibilityMask | ViewMask;
#else // DIM_MULTI_VIEW
	OutVisibilityMask = ViewMask;
#endif // DIM_MULTI_VIEW
}


EARLYDEPTHSTENCIL
void MainPS(
	in nointerpolation uint InstanceId : INSTANCE_ID,
	in nointerpolation uint VisibilityMask : VISIBILITY_MASK,
	out float4 OutColor : SV_Target0
)
{
	RWVisibilityMask[InstanceId] = VisibilityMask;
	OutColor = float4(1, 0, 0, 1);
}

// Debug utilities

Buffer<uint> InstanceOcclusionQueryBuffer;

void DebugMainVS(
	float3 InPosition : ATTRIBUTE0,
	uint InstanceId : SV_InstanceID,
	out nointerpolation uint OutDebugVisibility : INSTANCE_DEBUG_VISIBILITY,
	out float4 OutPosition : SV_POSITION
)
{
	ResolvedView = ResolveView();

	FInstanceSceneData InstanceData;
	EInstanceVisibilityStatus VisibilityStatus = GetInstanceDataAndVisibility(InstanceId, /*out*/ InstanceData);
	
	if (InstanceData.ValidInstance && VisibilityStatus != IVS_Incompatible)
	{
		float3 LocalPosition = (InPosition * (InstanceData.LocalBoundsExtent + OcclusionSlop) * InstanceData.DeterminantSign) + InstanceData.LocalBoundsCenter;
		float4 WorldPosition = TransformLocalToTranslatedWorld(LocalPosition, InstanceData.LocalToWorld);

		OutPosition = mul(WorldPosition, ResolvedView.TranslatedWorldToClip);

		const bool bWasVisibleLastFrame = (InstanceOcclusionQueryBuffer[InstanceId] & ViewMask) == ViewMask;

		if (VisibilityStatus == IVS_PossiblyVisible)
		{
			OutDebugVisibility = bWasVisibleLastFrame ? IVS_PossiblyVisible : IVS_Incompatible;
		}
		else
		{
			OutDebugVisibility = bWasVisibleLastFrame ? IVS_Visible : IVS_Hidden;
		}
	}
	else
	{
		OutPosition = asfloat(0xFFFFFFFF);
		OutDebugVisibility = IVS_Hidden;
	}
}

void DebugMainPS(
	in nointerpolation uint DebugVisibility : INSTANCE_DEBUG_VISIBILITY,
	out float4 OutColor : SV_Target0
)
{
	switch (DebugVisibility)
	{
	case IVS_Hidden:
		// Definitely invisible due to HZB test.
		// No occlusion query performed.
		OutColor = float4(1.0, 0.0, 0.0, 0.5); // red
		break;
	case IVS_Visible:
		// Definitely visible due to near plane intersection or other heuristics.
		// No occlusion query performed.
		OutColor = float4(1.0, 1.0, 0.0, 0.5); // yellow
		break;
	case IVS_PossiblyVisible:
		// Visible due to occlusion query.
		OutColor = float4(0.0, 1.0, 0.0, 0.2); // green
		break;
	case IVS_Incompatible:
		// Invisible due to occlusion query.
		OutColor = float4(1.0, 0.0, 1.0, 0.5); // magenta
		break;
	default:
		OutColor = (float4)0;
		break;
	}
}

