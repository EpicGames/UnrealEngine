// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "../Common.ush"

#include "InstanceCullingCommon.ush"

#include "InstanceCullingLoadBalancer.ush"

struct FInstanceCullingSetup
{
	FInstanceWorkSetup InstanceWorkSetup;
	bool bValid;
	bool bIsDynamic;
	uint InstanceId;
};

// InDynamicInstanceIdOffset/InDynamicInstanceIdMax are only needed if you want to support dynamic instances, otherwise you can provide 0
// ItemDataOffset is only needed if you work in batched mode where multiple load balancers are processed at the same time, see ENABLE_BATCH_MODE
FInstanceCullingSetup LoadInstanceCullingSetup(uint3 GroupId, uint GroupThreadIndex, uint InDynamicInstanceIdOffset, uint InDynamicInstanceIdMax, uint ItemDataOffset)
{
	FInstanceCullingSetup Result = (FInstanceCullingSetup)0;

	Result.InstanceWorkSetup = InstanceCullingLoadBalancer_Setup(GroupId, GroupThreadIndex, ItemDataOffset);

	if (!Result.InstanceWorkSetup.bValid)
	{
		return Result;
	}

	Result.bValid = true;
	Result.InstanceId = Result.InstanceWorkSetup.Item.InstanceDataOffset + uint(Result.InstanceWorkSetup.LocalItemIndex);
	// Check dynamic flag to offset the instance ID
	if (IsDynamicInstanceDataOffset(Result.InstanceWorkSetup.Item.Payload))
	{
		Result.InstanceId += InDynamicInstanceIdOffset;
		checkSlow(Result.InstanceId < InDynamicInstanceIdMax || (InDynamicInstanceIdOffset == 0 && InDynamicInstanceIdMax == 0));
		Result.bIsDynamic = true;
	}

	return Result;
}
