// Copyright Epic Games, Inc. All Rights Reserved..

/*=============================================================================
	InstancedStereo.usf: Resolve which view uniforms in a stereo pair to use.
=============================================================================*/

#pragma once

// Explictly include view uniform buffers
#include "/Engine/Generated/UniformBuffers/View.ush"
#include "/Engine/Generated/UniformBuffers/InstancedView.ush"

// ViewState, GetPrimaryView and GetInstancedView are generated by the shader compiler to ensure View uniform buffer changes are up to date.
// see GenerateInstancedStereoCode()
#include "/Engine/Generated/GeneratedInstancedStereo.ush"

// Define Texture2DMultiView if it hasn't already been defined in an automatically generated uniform buffer header
#ifndef Texture2DMultiView
	#if MOBILE_MULTI_VIEW
	#define Texture2DMultiView Texture2DArray
	#else
	#define Texture2DMultiView Texture2D
	#endif
#endif

void FinalizeViewState(inout ViewState InOutView)
{
	InOutView.WorldToClip = MakeDFInverseMatrix(InOutView.ViewOriginHigh, InOutView.RelativeWorldToClip);
	InOutView.ClipToWorld = MakeDFMatrix(InOutView.ViewOriginHigh, InOutView.ClipToRelativeWorld);
	InOutView.ScreenToWorld = MakeDFMatrix(InOutView.ViewOriginHigh, InOutView.ScreenToRelativeWorld);
	InOutView.PrevClipToWorld = MakeDFMatrix(InOutView.ViewOriginHigh, InOutView.PrevClipToRelativeWorld);

	InOutView.WorldCameraOrigin = MakeDFVector3(InOutView.ViewOriginHigh, InOutView.ViewOriginLow);
	InOutView.WorldViewOrigin = MakeDFVector3(InOutView.WorldViewOriginHigh, InOutView.WorldViewOriginLow);
	InOutView.PrevWorldCameraOrigin = MakeDFVector3(InOutView.PrevWorldCameraOriginHigh, InOutView.PrevWorldCameraOriginLow);
	InOutView.PrevWorldViewOrigin = MakeDFVector3(InOutView.PrevWorldViewOriginHigh, InOutView.PrevWorldViewOriginLow);
	InOutView.PreViewTranslation = MakeDFVector3(InOutView.PreViewTranslationHigh, InOutView.PreViewTranslationLow);
	InOutView.PrevPreViewTranslation = MakeDFVector3(InOutView.PrevPreViewTranslationHigh, InOutView.PrevPreViewTranslationLow);

#if VIEW_HAS_TILEOFFSET_DATA
	InOutView.TileOffset.WorldCameraOrigin = MakeLWCVector3(InOutView.ViewTilePosition, InOutView.RelativeWorldCameraOriginTO);
	InOutView.TileOffset.WorldViewOrigin = MakeLWCVector3(InOutView.ViewTilePosition, InOutView.RelativeWorldViewOriginTO);
	InOutView.TileOffset.PrevWorldCameraOrigin = MakeLWCVector3(InOutView.ViewTilePosition, InOutView.PrevRelativeWorldCameraOriginTO);
	InOutView.TileOffset.PrevWorldViewOrigin = MakeLWCVector3(InOutView.ViewTilePosition, InOutView.PrevRelativeWorldViewOriginTO);
	InOutView.TileOffset.PreViewTranslation = MakeLWCVector3(-InOutView.ViewTilePosition, InOutView.RelativePreViewTranslationTO);
	InOutView.TileOffset.PrevPreViewTranslation = MakeLWCVector3(-InOutView.ViewTilePosition, InOutView.RelativePrevPreViewTranslationTO);
#endif
}

#define PrimaryView GetPrimaryView()

static ViewState ResolvedView = (ViewState)0.0f;

ViewState ResolveView()
{
	return GetPrimaryView();
}

#if (INSTANCED_STEREO || MOBILE_MULTI_VIEW)
ViewState ResolveView(uint ViewIndex)
{
	return GetInstancedView(ViewIndex);
}
#endif

#if INSTANCED_STEREO
static const float EyeOffsetScale[2] = { -1.0, 1.0 };
static const float4 EyeClipEdge[2] = { float4(-1.0, 0.0, 0.0, 1.0), float4(1.0, 0.0, 0.0, 1.0) };
#endif

bool IsInstancedStereo()
{
#if INSTANCED_STEREO
	return (uint)InstancedView.StereoPassIndex[1] > 0;
#else
	return false;
#endif
}

uint GetEyeIndex(uint InstanceId)
{
#if INSTANCED_STEREO
	return IsInstancedStereo() ? InstanceId & 1 : 0;
#else
	return 0;
#endif
}

uint GetInstanceId(uint InstanceId)
{
#if INSTANCED_STEREO
	return IsInstancedStereo() ? InstanceId / 2 : InstanceId;
#else
	return InstanceId;
#endif
}

struct FStereoVSInput
{
#if INSTANCED_STEREO
    uint InstanceId : SV_InstanceID;
#elif MOBILE_MULTI_VIEW
    nointerpolation uint ViewId : SV_ViewID;
#endif
};

struct FStereoVSToPS
{
#if INSTANCED_STEREO 
	nointerpolation uint EyeIndex : VIEW_ID;
#endif
};

struct FStereoVSOutput
{
	FStereoVSToPS StereoInterpolants;
#if INSTANCED_STEREO 
	#if MOBILE_MULTI_VIEW
		uint TargetIndex : SV_RenderTargetArrayIndex;
	#else
		uint TargetIndex : SV_ViewPortArrayIndex;
	#endif
#endif
};

struct FStereoPSInput
{
#if MOBILE_MULTI_VIEW  && !INSTANCED_STEREO
	nointerpolation uint ViewId : SV_ViewID;
#endif
	FStereoVSToPS StereoInterpolants;
};

void StereoSetupVS(const uint EyeIndex, out FStereoVSOutput Output)
{
#if INSTANCED_STEREO
	Output.TargetIndex = EyeIndex;
	Output.StereoInterpolants.EyeIndex = EyeIndex;
	ResolvedView = ResolveView(EyeIndex);
#elif MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(EyeIndex);
#else
	ResolvedView = ResolveView();
#endif
}
	
void StereoSetupVS(FStereoVSInput StereoInput, out FStereoVSOutput Output)
{
#if INSTANCED_STEREO
	StereoSetupVS(GetEyeIndex(StereoInput.InstanceId), Output);
#elif MOBILE_MULTI_VIEW
	StereoSetupVS(StereoInput.ViewId, Output);
#else
	StereoSetupVS(0, Output);
#endif
}

void ScreenPassStereoOutputSetupVS(FStereoVSInput StereoInput, out FStereoVSOutput Output)
{
	// Screen passes are only drawn twice so the instance id corresponds to the eye index.
#if INSTANCED_STEREO
	Output.StereoInterpolants.EyeIndex = StereoInput.InstanceId;
	Output.TargetIndex = StereoInput.InstanceId;
#endif
}

void StereoSetupPS(FStereoPSInput StereoInput)
{
#if INSTANCED_STEREO
	ResolvedView = ResolveView(StereoInput.StereoInterpolants.EyeIndex);
#elif MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(StereoInput.ViewId);
#else
	ResolvedView = ResolveView();
#endif
	}

void StereoSetupCS(uint EyeIndex = 0)
{
#if INSTANCED_STEREO || MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(EyeIndex);
#else
	ResolvedView = ResolveView();
#endif
	}

uint GetEyeIndex(FStereoPSInput StereoInput)
{
#if INSTANCED_STEREO
	return StereoInput.StereoInterpolants.EyeIndex;
#elif MOBILE_MULTI_VIEW
	return StereoInput.ViewId;
#else
	return 0;
#endif
	}