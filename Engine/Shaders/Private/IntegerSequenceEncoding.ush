// Copyright Epic Games, Inc. All Rights Reserved.

// astc specification Integer Sequence Encoding
// https://registry.khronos.org/DataFormat/specs/1.3/dataformat.1.3.html#astc-integer-sequence-encoding

#include "BitPacking.ush"

// Quant  Levels
#define QUANT_2		0
#define QUANT_3		1
#define QUANT_4		2
#define QUANT_5		3
#define QUANT_6		4
#define QUANT_8		5
#define QUANT_10	6
#define QUANT_12	7
#define QUANT_16	8
#define QUANT_20	9
#define QUANT_24	10
#define QUANT_32	11
#define QUANT_40	12
#define QUANT_48	13
#define QUANT_64	14
#define QUANT_80	15
#define QUANT_96	16
#define QUANT_128	17
#define QUANT_160	18
#define QUANT_192	19
#define QUANT_256	20

// Bits Trits and Quints needed for quant encode
static const uint BitTritQuantCounts[21][3] =
{
  //Bits Trits Quants Level
	{ 1, 0, 0 }, //QUANT_2
	{ 0, 1, 0 }, //QUANT_3
	{ 2, 0, 0 }, //QUANT_4
	{ 0, 0, 1 }, //QUANT_5
	{ 1, 1, 0 }, //QUANT_6
	{ 3, 0, 0 }, //QUANT_8
	{ 1, 0, 1 }, //QUANT_10
	{ 2, 1, 0 }, //QUANT_12
	{ 4, 0, 0 }, //QUANT_16
	{ 2, 0, 1 }, //QUANT_20
	{ 3, 1, 0 }, //QUANT_24
	{ 5, 0, 0 }, //QUANT_32
	{ 3, 0, 1 }, //QUANT_40
	{ 4, 1, 0 }, //QUANT_48
	{ 6, 0, 0 }, //QUANT_64
	{ 4, 0, 1 }, //QUANT_80
	{ 5, 1, 0 }, //QUANT_96
	{ 7, 0, 0 }, //QUANT_128
	{ 5, 0, 1 }, //QUANT_160
	{ 6, 1, 0 }, //QUANT_192
	{ 8, 0, 0 }, //QUANT_256
	};

#define WEIGHT_QUANTIZE_NUM 32

uint IntegerFromTrits(uint Trits0, uint Trits1, uint Trits2, uint Trits3, uint Trits4)
{
	uint Index = Trits4 * 81u + Trits3 * 27u + Trits2 * 9u + Trits1 * 3u + Trits0;
	return AstcParameters.TritsToInteger[Index];
}

uint IntegerFromQuints(uint Quint0, uint Quint1, uint Quint2)
{
	uint Index = Quint2 * 25u + Quint1 * 5u + Quint0;
	return AstcParameters.QuintsToInteger[Index];
}

void GetBitTritQuantCounts(uint Range, out uint BitCount, out uint TritCount, out uint QuintCount)
{
	BitCount = BitTritQuantCounts[Range][0];
	TritCount = BitTritQuantCounts[Range][1];
	QuintCount = BitTritQuantCounts[Range][2];
}

void ExtractLeastSignificantBits(uint NumBits, uint Num, out uint RemainingBits, out uint OutLSBits)
{
	uint BitsMask = (1u << NumBits) - 1u;
	OutLSBits = Num & BitsMask;
	RemainingBits = (Num >> NumBits) & 0xFFu;
}

// Encodes 5 quantized numbers into trit representation, and writes to the output buffeer
void EncodeTrits(uint NumBits, uint Num0, uint Num1, uint Num2, uint Num3, uint Num4, inout uint4 OutBuffer, inout uint BufferOffset)
{
	uint TritBits0, TritBits1, TritBits2, TritBits3, TritBits4;
	uint LSBits0, LSBits1, LSBits2, LSBits3, LSBits4;

	ExtractLeastSignificantBits(NumBits, Num0, TritBits0, LSBits0);
	ExtractLeastSignificantBits(NumBits, Num1, TritBits1, LSBits1);
	ExtractLeastSignificantBits(NumBits, Num2, TritBits2, LSBits2);
	ExtractLeastSignificantBits(NumBits, Num3, TritBits3, LSBits3);
	ExtractLeastSignificantBits(NumBits, Num4, TritBits4, LSBits4);

	// Packed 8 bit representation of the 5 trits
	uint TritPack = IntegerFromTrits(TritBits0, TritBits1, TritBits2, TritBits3, TritBits4);

	const uint TritLayout[5][3] =
	{
	// Offset Mask, NumBits
		{ 0u, 3u, 2u },
		{ 2u, 3u, 2u },
		{ 4u, 1u, 1u },
		{ 5u, 3u, 2u },
		{ 7u, 1u, 1u },
	};

	uint TritPacks[5] = 
	{
		(TritPack >> TritLayout[0][0]) & TritLayout[0][1],
		(TritPack >> TritLayout[1][0]) & TritLayout[1][1],
		(TritPack >> TritLayout[2][0]) & TritLayout[2][1],
		(TritPack >> TritLayout[3][0]) & TritLayout[3][1],
		(TritPack >> TritLayout[4][0]) & TritLayout[4][1]
	};

	// Interleave the least significant bits with the trit pack value
	WriteBits(OutBuffer, BufferOffset, LSBits0, NumBits);
	WriteBits(OutBuffer, BufferOffset, TritPacks[0], TritLayout[0][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits1, NumBits);
	WriteBits(OutBuffer, BufferOffset, TritPacks[1], TritLayout[1][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits2, NumBits);
	WriteBits(OutBuffer, BufferOffset, TritPacks[2], TritLayout[2][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits3, NumBits);
	WriteBits(OutBuffer, BufferOffset, TritPacks[3], TritLayout[3][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits4, NumBits);
	WriteBits(OutBuffer, BufferOffset, TritPacks[4], TritLayout[4][2]);
}

// Encodes 3 quantized numbers into quints representation, and writes to the output buffeer
void EncodeQuints(uint NumBits, uint Num0, uint Num1, uint Num2, inout uint4 OutBuffer, inout uint BufferOffset)
{
	uint QuintBits0, QuintBits1, QuintBits2;
	uint LSBits0, LSBits1, LSBits2;

	ExtractLeastSignificantBits(NumBits, Num0, QuintBits0, LSBits0);
	ExtractLeastSignificantBits(NumBits, Num1, QuintBits1, LSBits1);
	ExtractLeastSignificantBits(NumBits, Num2, QuintBits2, LSBits2);

	// Packed 8 bit representation of the 3 quints
	uint QuintPack = IntegerFromQuints(QuintBits0, QuintBits1, QuintBits2);

	const uint QuintLayout[3][3] =
	{
	// Offset Mask, NumBits
		{ 0u, 7u, 3u },
		{ 3u, 3u, 2u },
		{ 5u, 3u, 2u },
	};

	uint QuintPacks[3] = 
	{
		(QuintPack >> QuintLayout[0][0]) & QuintLayout[0][1],
		(QuintPack >> QuintLayout[1][0]) & QuintLayout[1][1],
		(QuintPack >> QuintLayout[2][0]) & QuintLayout[2][1]
	};

	// Interleave the least significant bits with the quint pack value
	WriteBits(OutBuffer, BufferOffset, LSBits0, NumBits);
	WriteBits(OutBuffer, BufferOffset, QuintPacks[0], QuintLayout[0][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits1, NumBits);
	WriteBits(OutBuffer, BufferOffset, QuintPacks[1], QuintLayout[1][2]);

	WriteBits(OutBuffer, BufferOffset, LSBits2, NumBits);
	WriteBits(OutBuffer, BufferOffset, QuintPacks[2], QuintLayout[2][2]);
}


// Encodes 8 Numbers  
// TODO - This could be repurposed for other size counts by passing EncodeCount as a value.
//        This could also potentially be templated for any number count, but 8 and 16 work for basic u8 RGBA textures
void EncodeEndpointsQuant(uint Method, uint Numbers[8], inout uint4 OutBuffer, inout uint BufferOffset)
{
	const uint EncodeCount = 8u;

	uint BitCount; uint TritCount; uint QuintCount;
	GetBitTritQuantCounts(Method, BitCount, TritCount, QuintCount);

	uint InitialBufferOffset = BufferOffset;
	if(TritCount == 1u)
	{
		EncodeTrits(BitCount, Numbers[0], Numbers[1], Numbers[2], Numbers[3], Numbers[4], OutBuffer, BufferOffset);
		EncodeTrits(BitCount, Numbers[5], Numbers[6], Numbers[7], 0, 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial trits
		uint FullTritBlockSize = 8u + 5u * BitCount;
		uint FullBlockSize = FullTritBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 4u) / 5u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
		
	}
	else if(QuintCount == 1u)
	{
		EncodeQuints(BitCount, Numbers[0], Numbers[1], Numbers[2], OutBuffer, BufferOffset);
		EncodeQuints(BitCount, Numbers[3], Numbers[4], Numbers[5], OutBuffer, BufferOffset);
		EncodeQuints(BitCount, Numbers[6], Numbers[7], 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial quints
		uint FullQuintBlockSize = 7u + 3u * BitCount;
		uint FullBlockSize = FullQuintBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 2u) / 3u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
	}
	else
	{
		[unroll]
		for(uint Index = 0; Index < EncodeCount; ++Index)
		{
			WriteBits(OutBuffer, BufferOffset, Numbers[Index], BitCount);
		}
	}
}

void EncodeEndpointsQuant6(uint Method, uint Numbers[6], inout uint4 OutBuffer, inout uint BufferOffset)
{
	const uint EncodeCount = 6u;

	uint BitCount; uint TritCount; uint QuintCount;
	GetBitTritQuantCounts(Method, BitCount, TritCount, QuintCount);

	uint InitialBufferOffset = BufferOffset;
	if(TritCount == 1u)
	{
		EncodeTrits(BitCount, Numbers[0], Numbers[1], Numbers[2], Numbers[3], Numbers[4], OutBuffer, BufferOffset);
		EncodeTrits(BitCount, Numbers[5], 0, 0, 0, 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial trits
		uint FullTritBlockSize = 8u + 5u * BitCount;
		uint FullBlockSize = FullTritBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 4u) / 5u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
		
	}
	else if(QuintCount == 1u)
	{
		EncodeQuints(BitCount, Numbers[0], Numbers[1], Numbers[2], OutBuffer, BufferOffset);
		EncodeQuints(BitCount, Numbers[3], Numbers[4], Numbers[5], OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial quints
		uint FullQuintBlockSize = 7u + 3u * BitCount;
		uint FullBlockSize = FullQuintBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 2u) / 3u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
	}
	else
	{
		[unroll]
		for(uint Index = 0; Index < EncodeCount; ++Index)
		{
			WriteBits(OutBuffer, BufferOffset, Numbers[Index], BitCount);
		}
	}
}

// Encodes 16 Numbers
// TODO - This could be repurposed for other size counts by passing EncodeCount as a value.
//        This could also potentially be templated for any number count, but 8 and 16 work for basic u8 RGBA textures
void EncodeWeightsQuant16(uint Method, uint Numbers[16], inout uint4 OutBuffer, inout uint BufferOffset)
{
	const uint EncodeCount = 16u;

	uint BitCount; uint TritCount; uint QuintCount;
	GetBitTritQuantCounts(Method, BitCount, TritCount, QuintCount);

	uint InitialBufferOffset = BufferOffset;
	if(TritCount == 1u)
	{
		EncodeTrits(BitCount, Numbers[0], Numbers[1], Numbers[2], Numbers[3], Numbers[4], OutBuffer, BufferOffset);
		EncodeTrits(BitCount, Numbers[5], Numbers[6], Numbers[7], Numbers[8], Numbers[9], OutBuffer, BufferOffset);
		EncodeTrits(BitCount, Numbers[10], Numbers[11], Numbers[12], Numbers[13], Numbers[14], OutBuffer, BufferOffset);
		EncodeTrits(BitCount, Numbers[15], 0, 0, 0, 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial trits
		uint FullTritBlockSize = 8u + 5u * BitCount;
		uint FullBlockSize = FullTritBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 4u) / 5u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
		
	}
	else if(QuintCount == 1u)
	{
		EncodeQuints(BitCount, Numbers[0], Numbers[1], Numbers[2], OutBuffer, BufferOffset);
		EncodeQuints(BitCount, Numbers[3], Numbers[4], Numbers[5], OutBuffer, BufferOffset);
		EncodeQuints(BitCount, Numbers[6], Numbers[7], 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial quints
		uint FullQuintBlockSize = 7u + 3u * BitCount;
		uint FullBlockSize = FullQuintBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 2u) / 3u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
	}
	else
	{
		[unroll]
		for(uint Index = 0; Index < EncodeCount; ++Index)
		{
			WriteBits(OutBuffer, BufferOffset, Numbers[Index], BitCount);
		}
	}
}

void EncodeWeightsQuant36(uint Method, uint Numbers[36], inout uint4 OutBuffer, inout uint BufferOffset)
{
	const uint EncodeCount = 36u;

	uint BitCount; uint TritCount; uint QuintCount;
	GetBitTritQuantCounts(Method, BitCount, TritCount, QuintCount);

	uint InitialBufferOffset = BufferOffset;
	if(TritCount == 1u)
	{
        EncodeTrits(BitCount, Numbers[0], Numbers[1], Numbers[2], Numbers[3], Numbers[4], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[5], Numbers[6], Numbers[7], Numbers[8], Numbers[9], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[10], Numbers[11], Numbers[12], Numbers[13], Numbers[14], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[15], Numbers[16], Numbers[17], Numbers[18], Numbers[19], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[20], Numbers[21], Numbers[22], Numbers[23], Numbers[24], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[25], Numbers[26], Numbers[27], Numbers[28], Numbers[29], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[30], Numbers[31], Numbers[32], Numbers[33], Numbers[34], OutBuffer, BufferOffset);
        EncodeTrits(BitCount, Numbers[35], 0, 0, 0, 0, OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial trits
		uint FullTritBlockSize = 8u + 5u * BitCount;
		uint FullBlockSize = FullTritBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 4u) / 5u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
	}
	else if(QuintCount == 1u)
	{
        EncodeQuints(BitCount, Numbers[0], Numbers[1], Numbers[2], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[3], Numbers[4], Numbers[5], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[6], Numbers[7], Numbers[8], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[9], Numbers[10], Numbers[11], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[12], Numbers[13], Numbers[14], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[15], Numbers[16], Numbers[17], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[18], Numbers[19], Numbers[20], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[21], Numbers[22], Numbers[23], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[24], Numbers[25], Numbers[26], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[27], Numbers[28], Numbers[29], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[30], Numbers[31], Numbers[32], OutBuffer, BufferOffset);
        EncodeQuints(BitCount, Numbers[33], Numbers[34], Numbers[35], OutBuffer, BufferOffset);

		// The OutBuffer size needs to be truncated for partial quints
		uint FullQuintBlockSize = 7u + 3u * BitCount;
		uint FullBlockSize = FullQuintBlockSize * EncodeCount;
		uint TruncatedOffset = (FullBlockSize + 2u) / 3u;
		BufferOffset = InitialBufferOffset + TruncatedOffset;
	}
	else
	{
		[unroll]
		for(uint Index = 0; Index < EncodeCount; ++Index)
		{
			WriteBits(OutBuffer, BufferOffset, Numbers[Index], BitCount);
		}
	}
}


