// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

bool SphereIntersectCone(float4 SphereCenterAndRadius, float3 ConeVertex, float3 ConeAxis, float ConeAngleCos, float ConeAngleSin)
{
	float3 U = ConeVertex - (SphereCenterAndRadius.w / ConeAngleSin) * ConeAxis;
	float3 D = SphereCenterAndRadius.xyz - U;
	float DSizeSq = dot(D, D);
	float E = dot(ConeAxis, D);

	if (E > 0 && E * E >= DSizeSq * ConeAngleCos * ConeAngleCos)
	{
		D = SphereCenterAndRadius.xyz - ConeVertex;
		DSizeSq = dot(D, D);
		E = -dot(ConeAxis, D);

		if (E > 0 && E * E >= DSizeSq * ConeAngleSin * ConeAngleSin)
		{
			return DSizeSq <= SphereCenterAndRadius.w * SphereCenterAndRadius.w;
		}
		else
		{
			return true;
		}
	}

	return false;
}

bool SphereIntersectConeWithMaxDistance(float4 SphereCenterAndRadius, float3 ConeVertex, float3 ConeAxis, float ConeAngleCos, float ConeAngleSin, float MaxDistanceAlongAxis)
{
	if (SphereIntersectCone(SphereCenterAndRadius, ConeVertex, ConeAxis, ConeAngleCos, ConeAngleSin))
	{
		float ConeAxisDistance = dot(SphereCenterAndRadius.xyz - ConeVertex, ConeAxis);
		float ConeAxisDistanceMax = ConeAxisDistance - SphereCenterAndRadius.w;

		return ConeAxisDistanceMax < MaxDistanceAlongAxis;
	}

	return false;
}

bool SphereIntersectConeWithDepthRanges(float4 SphereCenterAndRadius, float3 ConeVertex, float3 ConeAxis, float ConeAngleCos, float ConeAngleSin, float4 ConeAxisDepthRanges)
{
	if (SphereIntersectCone(SphereCenterAndRadius, ConeVertex, ConeAxis, ConeAngleCos, ConeAngleSin))
	{
		float ConeAxisDistance = dot(SphereCenterAndRadius.xyz - ConeVertex, ConeAxis);
		float2 ConeAxisDistanceMinMax = float2(ConeAxisDistance + SphereCenterAndRadius.w, ConeAxisDistance - SphereCenterAndRadius.w);

		if ((ConeAxisDistanceMinMax.x > ConeAxisDepthRanges.x && ConeAxisDistanceMinMax.y < ConeAxisDepthRanges.y)
			|| (ConeAxisDistanceMinMax.x > ConeAxisDepthRanges.z && ConeAxisDistanceMinMax.y < ConeAxisDepthRanges.w))
		{
			return true;
		}
	}

	return false;
}

bool SphereIntersectSphere(float4 SphereCenterAndRadius, float4 OtherSphereCenterAndRadius)
{
	float CombinedRadii = SphereCenterAndRadius.w + OtherSphereCenterAndRadius.w;
	float3 VectorBetweenCenters = SphereCenterAndRadius.xyz - OtherSphereCenterAndRadius.xyz;
	return dot(VectorBetweenCenters, VectorBetweenCenters) < CombinedRadii * CombinedRadii;
}

// Given two ranges of distances along a ray - check if they overlap
// NOTE: This check ignores empty intervals and intervals which contain only a single point
bool CheckOverlap(float2 Ta, float2 Tb) {
	return max(Ta.x, Tb.x) < min(Ta.y, Tb.y);
}

bool RayOverlapsAABB(float3 O, float3 D, float TMax, float3 Lo, float3 Hi)
{
	float3 InvRayDir = rcp(D);
	float3 T0 = (Lo - O) * InvRayDir;
	float3 T1 = (Hi - O) * InvRayDir;
	float3 TNear = min(T0, T1), TFar = max(T0, T1);
	return max(max(TNear.x, TNear.y), max(TNear.z, 0.0)) <= min(min(TFar.x, TFar.y), min(TFar.z, TMax));
}

// Compute the overlap between an infinite ray and an infinite cone volume
// The Apex is the origin of the cone, the normal is the unit vector , CosTheta is the half-angle describing the cone's opening
float2 RayInfiniteConeVolumeOverlap(
	float3 RayOrigin, float3 RayDirection,
	float3 Apex, float3 Axis,
	float CosTheta)
{
	// Adapted from: https://www.iquilezles.org/www/articles/intersectors/intersectors.htm
	const float3 ba = Axis * CosTheta;
	const float3 oa = RayOrigin - Apex;
	const float3 rd = RayDirection;

	const float m0 = dot(ba, ba);
	const float m1 = dot(oa, ba);
	const float m3 = dot(rd, ba);

	const float m4 = dot(rd, oa);
	const float m5 = dot(oa, oa);

	const float k2 = m0 * m0 - m3 * m3;
	const float k1 = m0 * m0 * m4 - m1 * m3;
	const float k0 = m0 * m0 * m5 - m1 * m1;

	const float h = k1 * k1 - k2 * k0;
	if (h > 0)
	{
		const float s = sign(k2);
		const float2 t = (-k1 + float2(-s, s) * sqrt(h)) / k2;
		const float2 y = m1 + t * m3;

		// check which hits are valid:
		const int c = ((y.x > 0) ? 1 : 0) + (y.y > 0 ? 2 : 0);
		if (c == 1)
		{
			// only smaller root is valid, far root is invalid
			return float2(0.0, t.x);
		}
		if (c == 2)
		{
			// only larger root is valid, must have started outside the cone
			return float2(t.y, POSITIVE_INFINITY);
		}
		if (c == 3)
		{
			// both roots are valid
			return t;
		}
	}
	return -1.0;
}

float2 RaySphereOverlap(float3 Ro, float3 Rd, float3 Center, float Radius, float TMin, float TMax)
{
	float3 oc = Ro - Center;
	float b = dot(oc, Rd);
	float h = Radius * Radius - length2(oc - b * Rd);
	if (h > 0)
	{
		float2 t = -b + float2(-1, +1) * sqrt(h);
		return float2(max(t.x, TMin), min(t.y, TMax));
	}
	return -1.0;
}

// Check for overlap between an AABB and a Cone (the portion of the cone inside a sphere of the given Radius)
// NOTE: This test can be expensive, so one should check for AABB vs sphere overlap first
// NOTE: The last argument can be used to do a cheaper test using the AABB of the rounded cone
bool AABBOverlapsCurvedCone(float3 Lo, float3 Hi, float3 Apex, float3 Axis, float CosTheta, float Radius, bool bUseApproxTest)
{
	if (bUseApproxTest)
	{
		// Compute AABB of the cone, and check for overlap of the two AABBs
		// box around ray from light center to tip of the cone
		float3 Tip = Apex + Axis * Radius;
		float3 LightBoundLo = min(Apex, Tip);
		float3 LightBoundHi = max(Apex, Tip);

		// expand by disc around the farthest part of the cone
		float SinTheta2 = 1 - CosTheta * CosTheta;
		float3 Disc = sqrt(saturate(SinTheta2 * (1.0 - Axis * Axis)));
		LightBoundLo = min(LightBoundLo, Apex + Radius * (Axis * CosTheta - Disc));
		LightBoundHi = max(LightBoundHi, Apex + Radius * (Axis * CosTheta + Disc));

		float3 F = select(abs(Axis) > CosTheta, Apex + Radius * sign(Axis), Apex); // include far point along axis if it lies inside the cone
		LightBoundLo = min(LightBoundLo, F);
		LightBoundHi = max(LightBoundHi, F);

		// intersect bounds and see if we have anything left
		return all(max(Lo, LightBoundLo) < min(Hi, LightBoundHi));
	}
	else
	{
		return
			// Does the Apex line inside the AABB?
			all((Lo <= Apex) & (Apex <= Hi)) ||
			// Does the central axis of the cone pass overlap the AABB?
			RayOverlapsAABB(Apex, Axis, Radius, Lo, Hi) ||
			// Test each of the 12 AABB edges against cone -- if any of them overlap, we can stop
			// NOTE: this looks expensive, but there is a huge amount of simplification and common terms between each of these calls
			// NOTE: This is a volumetric test, so edges entirely _inside_ the cone, will pass the test
			// Check X edges
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Lo.y, Lo.z), float3(1, 0, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Lo.y, Lo.z), float3(1, 0, 0), Apex, Radius, 0, Hi.x - Lo.x)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Lo.y, Hi.z), float3(1, 0, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Lo.y, Hi.z), float3(1, 0, 0), Apex, Radius, 0, Hi.x - Lo.x)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Hi.y, Lo.z), float3(1, 0, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Hi.y, Lo.z), float3(1, 0, 0), Apex, Radius, 0, Hi.x - Lo.x)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Hi.y, Hi.z), float3(1, 0, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Hi.y, Hi.z), float3(1, 0, 0), Apex, Radius, 0, Hi.x - Lo.x)) ||
			// Check Y edges
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Lo.y, Lo.z), float3(0, 1, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Lo.y, Lo.z), float3(0, 1, 0), Apex, Radius, 0, Hi.y - Lo.y)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Lo.y, Hi.z), float3(0, 1, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Lo.y, Hi.z), float3(0, 1, 0), Apex, Radius, 0, Hi.y - Lo.y)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Hi.x, Lo.y, Lo.z), float3(0, 1, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Hi.x, Lo.y, Lo.z), float3(0, 1, 0), Apex, Radius, 0, Hi.y - Lo.y)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Hi.x, Lo.y, Hi.z), float3(0, 1, 0), Apex, Axis, CosTheta), RaySphereOverlap(float3(Hi.x, Lo.y, Hi.z), float3(0, 1, 0), Apex, Radius, 0, Hi.y - Lo.y)) ||
			// Check Z edges
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Lo.y, Lo.z), float3(0, 0, 1), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Lo.y, Lo.z), float3(0, 0, 1), Apex, Radius, 0, Hi.z - Lo.z)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Lo.x, Hi.y, Lo.z), float3(0, 0, 1), Apex, Axis, CosTheta), RaySphereOverlap(float3(Lo.x, Hi.y, Lo.z), float3(0, 0, 1), Apex, Radius, 0, Hi.z - Lo.z)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Hi.x, Lo.y, Lo.z), float3(0, 0, 1), Apex, Axis, CosTheta), RaySphereOverlap(float3(Hi.x, Lo.y, Lo.z), float3(0, 0, 1), Apex, Radius, 0, Hi.z - Lo.z)) ||
			CheckOverlap(RayInfiniteConeVolumeOverlap(float3(Hi.x, Hi.y, Lo.z), float3(0, 0, 1), Apex, Axis, CosTheta), RaySphereOverlap(float3(Hi.x, Hi.y, Lo.z), float3(0, 0, 1), Apex, Radius, 0, Hi.z - Lo.z));
	}
}

// Test AABB against solid sphere
bool AABBOverlapsSphere(float3 Lo, float3 Hi, float3 Center, float Radius)
{
	// James Arvo - Graphics Gems (p. 335)
	return length2(max(Lo - Center, 0) + max(Center - Hi, 0)) < Radius * Radius;
}

// Does the AABB lie fully "behind" the plane defined by Center and Normal
// NOTE: Normal vector does not need to be normalized
bool IsBoxBehindPlane(float3 Lo, float3 Hi, float3 Normal, float3 Center)
{
	return dot(select(Normal > 0, Hi, Lo) - Center, Normal) < 0;
}

// Test AABB against the penumbra frustum defined by a rectlight's barndoors
// This test is expensive, so should only be used if the barndoors are active
// It is assumed the AABB already overlaps the sphere of influence of the rect light and is at least partially "in front" of the light
// The last portion of the test is a bit more expensive, so skip it optionally (at the risk of more false-positives)
bool AABBOverlapsRectLightFrustum(float3 Lo, float3 Hi, float3 Center, float3 Normal, float3 Tangent, float HalfWidth, float HalfHeight, float Radius, float BarnCos, float BarnLen, bool bImprovedFrustumTest)
{
	const float3 dPdv = Tangent;
	const float3 dPdu = cross(Normal, dPdv);
	const float BarnSin = sqrt(1 - BarnCos * BarnCos);

	float3 BoundingPlane = float3(
		2 * HalfWidth  + BarnLen * BarnSin,
		2 * HalfHeight + BarnLen * BarnSin,
		BarnLen * BarnCos
	);
	// Test if box is outside any of the 4 barndoor penumbra planes:
	if (IsBoxBehindPlane(Lo, Hi, BoundingPlane.x * Normal + BoundingPlane.z * dPdu, Center - dPdu * HalfWidth) ||
		IsBoxBehindPlane(Lo, Hi, BoundingPlane.x * Normal - BoundingPlane.z * dPdu, Center + dPdu * HalfWidth) ||
		IsBoxBehindPlane(Lo, Hi, BoundingPlane.y * Normal + BoundingPlane.z * dPdv, Center - dPdv * HalfHeight) ||
		IsBoxBehindPlane(Lo, Hi, BoundingPlane.y * Normal - BoundingPlane.z * dPdv, Center + dPdv * HalfHeight))
	{
		return false;
	}
	
	if (!bImprovedFrustumTest)
	{
		return true;
	}

	// Checking the AABB against the 4 planes is not sufficient, and some cases which still lie outside could still pass through
	// See description here: https://iquilezles.org/articles/frustumcorrect/

	// Normalize the penumbra vector and scale it to the back of the sphere of influence
	BoundingPlane = normalize(BoundingPlane);
	BoundingPlane.xy *= Radius * rcp(BoundingPlane.z);
	// Get 8 corners of the penumbra frustum in world space
	const float3 C00 = Center - HalfWidth * dPdu - HalfHeight * dPdv, F00 = C00 - BoundingPlane.x * dPdu - BoundingPlane.y * dPdv + Radius * Normal;
	const float3 C01 = Center + HalfWidth * dPdu + HalfHeight * dPdv, F01 = C01 - BoundingPlane.x * dPdu + BoundingPlane.y * dPdv + Radius * Normal;
	const float3 C10 = Center - HalfWidth * dPdu - HalfHeight * dPdv, F10 = C10 + BoundingPlane.x * dPdu - BoundingPlane.y * dPdv + Radius * Normal;
	const float3 C11 = Center + HalfWidth * dPdu + HalfHeight * dPdv, F11 = C11 + BoundingPlane.x * dPdu + BoundingPlane.y * dPdv + Radius * Normal;
	// Get an AABB around the 8 frustum points and check if it overlaps the query AABB
	if (any(max(max(max(C00, F00), max(C01, F01)), max(max(C10, F10), max(C11, F11))) < Lo) ||
		any(min(min(min(C00, F00), min(C01, F01)), min(min(C10, F10), min(C11, F11))) > Hi))
	{
		return false;
	}
	return true;
}