// Copyright Epic Games, Inc. All Rights Reserved.

// LWCNormalizeTile will convert a LWC value to have an integral tile value, and an offset value within (-TileSize, TileSize)
// This is important if we want to compare LWC values, or to ensure that fractional part is fully contained within the offset
FLWCType LWCNormalizeTile(FLWCType V)
{
	FFloatType IntTile = floor(V.Tile + (V.Offset * UE_LWC_RENDER_TILE_SIZE_RCP + 0.5f));
	return LWCConstructor(IntTile, (V.Tile - IntTile) * UE_LWC_RENDER_TILE_SIZE + V.Offset);
}

// Forces the tile to be an integral value, but doesn't adjust range of offset
FLWCType LWCMakeIntTile(FLWCType V)
{
	FFloatType IntTile = floor(V.Tile);
	return LWCConstructor(IntTile, (V.Tile - IntTile) * UE_LWC_RENDER_TILE_SIZE + V.Offset);
}

FFloatType LWCSqrtUnscaled(FLWCType V) { return sqrt(V.Offset * UE_LWC_RENDER_TILE_SIZE_RCP + LWCGetTile(V)); }
FFloatType LWCRsqrtUnscaled(FLWCType V) { return rsqrt(V.Offset * UE_LWC_RENDER_TILE_SIZE_RCP + LWCGetTile(V)); }
FFloatType LWCRcpUnscaled(FLWCType V) { return rcp(V.Offset * UE_LWC_RENDER_TILE_SIZE_RCP + LWCGetTile(V)); }
FFloatType LWCSqrtScaled(FLWCType V, float Scale) { return LWCSqrtUnscaled(V) * Scale; }
FFloatType LWCRsqrtScaled(FLWCType V, float Scale) { return LWCRsqrtUnscaled(V) * Scale; }
FFloatType LWCRcpScaled(FLWCType V, float Scale) { return LWCRcpUnscaled(V) * Scale; }
FFloatType LWCSqrt(FLWCType V) { return LWCSqrtScaled(V, UE_LWC_RENDER_TILE_SIZE_SQRT); }
FFloatType LWCRsqrt(FLWCType V) { return LWCRsqrtScaled(V, UE_LWC_RENDER_TILE_SIZE_RSQRT); }
FFloatType LWCRcp(FLWCType V) { return LWCRcpScaled(V, UE_LWC_RENDER_TILE_SIZE_RCP); }

#define LWC_COMPARE_OP(Name, Op, FLWCType, FFloatType, FBoolType) \
	FBoolType Name(FLWCType Lhs, FLWCType Rhs) { \
		return (LWCGetTile(Lhs) - LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE Op Rhs.Offset - Lhs.Offset; } \
	FBoolType Name(FFloatType Lhs, FLWCType Rhs) { \
		return -LWCGetTile(Rhs) * UE_LWC_RENDER_TILE_SIZE Op Rhs.Offset - Lhs; } \
	FBoolType Name(FLWCType Lhs, FFloatType Rhs) { \
		return LWCGetTile(Lhs) * UE_LWC_RENDER_TILE_SIZE Op Rhs - Lhs.Offset; }

LWC_COMPARE_OP(LWCGreater, >, FLWCType, FFloatType, FBoolType)
LWC_COMPARE_OP(LWCGreaterEqual, >=, FLWCType, FFloatType, FBoolType)
LWC_COMPARE_OP(LWCLess, <, FLWCType, FFloatType, FBoolType)
LWC_COMPARE_OP(LWCLessEqual, <=, FLWCType, FFloatType, FBoolType)

#undef LWC_COMPARE_OP

// No ddxy inside ray tracing shaders
#if RAYHITGROUPSHADER || RAYMISSHADER || RAYCALLABLESHADER || USE_FORCE_TEXTURE_MIP
FFloatType LWCDdx(FLWCType V) { return (FFloatType)0; }
FFloatType LWCDdy(FLWCType V) { return (FFloatType)0; }
#else
FFloatType LWCDdx(FLWCType V) { return ddx(LWCGetTile(V)) * UE_LWC_RENDER_TILE_SIZE + ddx(V.Offset); }
FFloatType LWCDdy(FLWCType V) { return ddy(LWCGetTile(V)) * UE_LWC_RENDER_TILE_SIZE + ddy(V.Offset); }
#endif

FLWCType LWCAdd(FLWCType Lhs, FLWCType Rhs) { return LWCConstructor(LWCGetTile(Lhs) + LWCGetTile(Rhs), Lhs.Offset + Rhs.Offset); }
FLWCType LWCAdd(FFloatType Lhs, FLWCType Rhs) { return LWCConstructor(LWCGetTile(Rhs), Lhs + Rhs.Offset); }
FLWCType LWCAdd(FLWCType Lhs, FFloatType Rhs) { return LWCConstructor(LWCGetTile(Lhs), Lhs.Offset + Rhs); }

FLWCType LWCSubtract(FLWCType Lhs, FLWCType Rhs) { return LWCConstructor(LWCGetTile(Lhs) - LWCGetTile(Rhs), Lhs.Offset - Rhs.Offset); }
FLWCType LWCSubtract(FFloatType Lhs, FLWCType Rhs) { return LWCConstructor(-LWCGetTile(Rhs), Lhs - Rhs.Offset); }
FLWCType LWCSubtract(FLWCType Lhs, FFloatType Rhs) { return LWCConstructor(LWCGetTile(Lhs), Lhs.Offset - Rhs); }

FBoolType LWCEquals(FLWCType Lhs, FLWCType Rhs)
{
	return (LWCGetTile(Lhs) - LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE == Rhs.Offset - Lhs.Offset;
}
FBoolType LWCEquals(FFloatType Lhs, FLWCType Rhs)
{
	return -LWCGetTile(Rhs) * UE_LWC_RENDER_TILE_SIZE == Rhs.Offset - Lhs;
} 
FBoolType LWCEquals(FLWCType Lhs, FFloatType Rhs)
{
	return LWCGetTile(Lhs) * UE_LWC_RENDER_TILE_SIZE == Rhs - Lhs.Offset;
}
FBoolType LWCEqualsApprox(FLWCType Lhs, FLWCType Rhs, float Threshold)
{
	return abs((LWCGetTile(Lhs) - LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE + (Lhs.Offset - Rhs.Offset)) < (FFloatType)Threshold;
}
FBoolType LWCEqualsApprox(FFloatType Lhs, FLWCType Rhs, float Threshold)
{
	return abs(-LWCGetTile(Rhs) * UE_LWC_RENDER_TILE_SIZE + (Lhs - Rhs.Offset)) < (FFloatType)Threshold;
}
FBoolType LWCEqualsApprox(FLWCType Lhs, FFloatType Rhs, float Threshold)
{
	return abs(LWCGetTile(Lhs) * UE_LWC_RENDER_TILE_SIZE + (Lhs.Offset - Rhs)) < (FFloatType)Threshold;
}

// Needed for new translator
FBoolType LWCNotEquals(FLWCType Lhs, FLWCType Rhs)
{
	return (LWCGetTile(Lhs) - LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE != Rhs.Offset - Lhs.Offset;
}

FLWCType LWCSelect(FBoolType S, FLWCType Lhs, FLWCType Rhs) { return LWCConstructor(select(S, LWCGetTile(Lhs), LWCGetTile(Rhs)), select(S, Lhs.Offset, Rhs.Offset)); }
FLWCType LWCSelect(FBoolType S, FFloatType Lhs, FLWCType Rhs) { return LWCConstructor(select(S, (FFloatType)0, LWCGetTile(Rhs)), select(S, Lhs, Rhs.Offset)); }
FLWCType LWCSelect(FBoolType S, FLWCType Lhs, FFloatType Rhs) { return LWCConstructor(select(S, LWCGetTile(Lhs), (FFloatType)0), select(S, Lhs.Offset, Rhs)); }

FLWCType LWCNegate(FLWCType V) { return LWCConstructor(-LWCGetTile(V), -V.Offset); }

FFloatType LWCFrac(FLWCType V)
{
	FFloatType FracTile = frac(LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE);
	return frac(FracTile + V.Offset);
}

FLWCType LWCFloor(FLWCType V) { FLWCType VN = LWCMakeIntTile(V); return LWCConstructor(LWCGetTile(VN), floor(VN.Offset)); }
FLWCType LWCCeil(FLWCType V) { FLWCType VN = LWCMakeIntTile(V); return LWCConstructor(LWCGetTile(VN), ceil(VN.Offset)); }
FLWCType LWCRound(FLWCType V) { FLWCType VN = LWCMakeIntTile(V); return LWCConstructor(LWCGetTile(VN), round(VN.Offset)); }
FLWCType LWCTrunc(FLWCType V) { FLWCType VN = LWCMakeIntTile(V); return LWCConstructor(LWCGetTile(VN), trunc(VN.Offset)); }

// These rely on +/-inf being handled properly by various min/max operations
FFloatType LWCSign(FLWCType V) { return FFloatType(sign(LWCToFloat(V))); }
FFloatType LWCSaturate(FLWCType V) { return saturate(LWCToFloat(V)); }
FFloatType LWCClampScalar(FLWCType V, float Low, float High) { return clamp(LWCToFloat(V), Low, High); }

FLWCType LWCMultiply(FLWCType Lhs, FLWCType Rhs)
{
	return LWCConstructor(LWCGetTile(Lhs) * (LWCGetTile(Rhs) * UE_LWC_RENDER_TILE_SIZE + Rhs.Offset) + LWCGetTile(Rhs) * Lhs.Offset, Lhs.Offset * Rhs.Offset);
}
FLWCType LWCMultiply(FFloatType Lhs, FLWCType Rhs) { return LWCConstructor(LWCGetTile(Rhs) * Lhs, Lhs * Rhs.Offset); }
FLWCType LWCMultiply(FLWCType Lhs, FFloatType Rhs) { return LWCConstructor(LWCGetTile(Lhs) * Rhs, Lhs.Offset * Rhs); }

FLWCType LWCDivide(FLWCType Lhs, FLWCType Rhs) { return LWCMultiply(Lhs, LWCRcp(Rhs)); }
FLWCType LWCDivide(FLWCType Lhs, FFloatType Rhs) { return LWCMultiply(Lhs, rcp(Rhs)); }
FLWCType LWCDivide(FFloatType Lhs, FLWCType Rhs) { return LWCConstructor((FFloatType)0, Lhs * LWCRcp(Rhs)); }

//FLWCType LWCLerp(FLWCType Lhs, FLWCType Rhs, FFloatType S) { return LWCAdd(Lhs, LWCMultiply(S, LWCSubtract(Rhs, Lhs))); }
FLWCType LWCLerp(FLWCType Lhs, FLWCType Rhs, FFloatType S)
{
	return LWCConstructor(lerp(LWCGetTile(Lhs), LWCGetTile(Rhs), S), lerp(Lhs.Offset, Rhs.Offset, S));
}

FFloatType LWCModulo(FLWCType Lhs, FFloatType Rhs)
{
	return LWCToFloat(LWCSubtract(Lhs, LWCMultiply(LWCTrunc(LWCDivide(Lhs, Rhs)), Rhs)));
}

FFloatType LWCFmod(FLWCType Lhs, FFloatType Rhs)
{
	return LWCToFloat(LWCSubtract(Lhs, LWCMultiply(LWCTrunc(LWCDivide(Lhs, Rhs)), Rhs)));
	//FFloatType ModTileSize = fmod(UE_LWC_RENDER_TILE_SIZE, Rhs);
	//return fmod(ModTileSize * LWCGetTile(Lhs) + Lhs.Offset, Rhs);
}
FFloatType LWCFmodFloor(FLWCType Lhs, FFloatType Rhs)
{
	return LWCToFloat(LWCSubtract(Lhs, LWCMultiply(LWCFloor(LWCDivide(Lhs, Rhs)), Rhs)));
	//FFloatType ModTileSize = FmodFloor(UE_LWC_RENDER_TILE_SIZE, Rhs);
	//return FmodFloor(LWCGetTile(Lhs) * ModTileSize + Lhs.Offset, Rhs);
}
FFloatType LWCFmodFloorPI(FLWCType V)
{
	return LWCFmodFloor(V, PI);
	//return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE_FMOD_PI + V.Offset;
}
FFloatType LWCFmodFloor2PI(FLWCType V)
{
	return LWCFmodFloor(V, 2.0f * PI);
	//return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE_FMOD_2PI + V.Offset;
}

FFloatType LWCSin(FLWCType V) { return sin(LWCFmodFloor2PI(V)); }
FFloatType LWCCos(FLWCType V) { return cos(LWCFmodFloor2PI(V)); }
FFloatType LWCTan(FLWCType V) { return tan(LWCFmodFloorPI(V)); }
FFloatType LWCASin(FLWCType V) { return asin(LWCClampScalar(V, -1.0f, 1.0f)); }
FFloatType LWCACos(FLWCType V) { return acos(LWCClampScalar(V, -1.0f, 1.0f)); }
FFloatType LWCATan(FLWCType V) { return atan(LWCClampScalar(V, -0.5f*PI, 0.5f*PI)); }

FFloatType LWCSmoothStep(FLWCType Lhs, FLWCType Rhs, FLWCType S)
{
	FFloatType t = LWCSaturate(LWCDivide(LWCSubtract(S, Lhs), LWCSubtract(Rhs, Lhs)));
	return t*t*(3.0f - (2.0f*t));
}

FLWCType LWCMin(FLWCType Lhs, FLWCType Rhs) { return LWCSelect(LWCLess(Lhs, Rhs), Lhs, Rhs); }
FLWCType LWCMin(FFloatType Lhs, FLWCType Rhs) { return LWCSelect(LWCLess(Lhs, Rhs), Lhs, Rhs); }
FLWCType LWCMin(FLWCType Lhs, FFloatType Rhs) { return LWCSelect(LWCLess(Lhs, Rhs), Lhs, Rhs); }
FLWCType LWCMax(FLWCType Lhs, FLWCType Rhs) { return LWCSelect(LWCGreater(Lhs, Rhs), Lhs, Rhs); }
FLWCType LWCMax(FFloatType Lhs, FLWCType Rhs) { return LWCSelect(LWCGreater(Lhs, Rhs), Lhs, Rhs); }
FLWCType LWCMax(FLWCType Lhs, FFloatType Rhs) { return LWCSelect(LWCGreater(Lhs, Rhs), Lhs, Rhs); }

// Needed for new translator
FLWCType LWCClamp(FLWCType X, FLWCType A, FLWCType B)
{
    return LWCMin(LWCMax(X, A), B);
}

FLWCType LWCAbs(FLWCType V) { return LWCSelect(LWCLess(V, (FFloatType)0), LWCNegate(V), V); }

FFloatType LWCStep(FLWCType Lhs, FLWCType Rhs)   { return select(LWCGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }
FFloatType LWCStep(FLWCType Lhs, FFloatType Rhs) { return select(LWCGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }
FFloatType LWCStep(FFloatType Lhs, FLWCType Rhs) { return select(LWCGreaterEqual(Rhs, Lhs), (FFloatType)1.0f, (FFloatType)0.0f); }

// LWCSquareScaled returns the value squared, scaled by the LWC tile size
FLWCType LWCSquareScaled(FLWCType V)
{
	FFloatType OffsetScaled = V.Offset * UE_LWC_RENDER_TILE_SIZE_RCP;
	return LWCConstructor(LWCGetTile(V) * (LWCGetTile(V) + OffsetScaled * 2.0f), V.Offset * OffsetScaled);
}
