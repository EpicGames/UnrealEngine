// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** EHeightmapAlphaFlags enum */
#define EHEIGHTMAPALPHAFLAGS_NONE 0
#define EHEIGHTMAPALPHAFLAGS_ADDITIVE 0
#define EHEIGHTMAPALPHAFLAGS_MIN (1 << 0)
#define EHEIGHTMAPALPHAFLAGS_MAX (1 << 1)
#define EHEIGHTMAPALPHAFLAGS_ALPHABLEND (EHEIGHTMAPALPHAFLAGS_MIN | EHEIGHTMAPALPHAFLAGS_MAX)

/** EWeightmapAlphaFlags enum */
#define EWEIGHTMAPALPHAFLAGS_NONE 0
#define EWEIGHTMAPALPHAFLAGS_ADDITIVE 0
#define EWEIGHTMAPALPHAFLAGS_MIN (1 << 0)
#define EWEIGHTMAPALPHAFLAGS_MAX (1 << 1)
#define EWEIGHTMAPALPHAFLAGS_ALPHABLEND (EHEIGHTMAPALPHAFLAGS_MIN | EHEIGHTMAPALPHAFLAGS_MAX)

// Must match LandscapeDataAccess::MaxValue : 
static const float LANDSCAPE_MAX_VALUE = 65535.0f;
// Must match LandscapeDataAccess::MidValue : 
static const float LANDSCAPE_MID_VALUE = 32768.0f;

static const uint INDEX_NONE = uint(-1);

// Height is packed from a [0, 2^16-1] value to two bytes, each of which we need to provide as a 0-1 float for writing 
float2 PackHeight(float InHeight)
{
	uint PackedHeight = (uint) clamp(round(InHeight), 0.0f, 65535.0f);
	return float2((float)(PackedHeight >> 8) / 255.0, (float)(PackedHeight & 0xFF) / 255.0);
}

float UnpackHeight(float2 InPackedHeight)
{
	// We used to do the inverse of PackHeight here, like this:
	// return float(((int) round(InPackedHeight.r * 255.0) << 8) | (int) round(InPackedHeight.g * 255.0));
	
	// However, this isn't great if you just interpolate between packed heights because that truncates the interpolation
	// of the higher order byte. So instead we do it by multiplication, which ends up simpler.
	return (InPackedHeight.r * 256 + InPackedHeight.g) * 255;
}

// Height alpha is 14 bits-wide + 2 bits are left for flags. InAlpha is expected in the [0,1] range
float2 PackHeightAlpha(float InAlpha, uint InAlphaFlags)
{
	uint PackedAlpha = (uint)clamp(InAlpha * 65535.0f, 0.0f, 65535.0f); // First 14 bits are used to store alpha (0xFFFC in float)
	uint PackedFlags = InAlphaFlags & 0x3; // Last 2 bits are used to store flags
	return float2((float)(PackedAlpha >> 8) / 255.0f, (float)((PackedAlpha & 0xFC) | PackedFlags) / 255.0);
}

void UnpackHeightAlpha(float2 InPackedAlpha, out float OutAlpha, out uint OutFlags)
{
	uint AlphaWithFlags = ((uint)round(InPackedAlpha.r * 255.0f) << 8) | (uint)round(InPackedAlpha.g * 255.0f);
	// The first 14 bits are used to store alpha
	uint ExtractedAlpha = (AlphaWithFlags >> 2);
	OutAlpha = (float)ExtractedAlpha / (float)((1 << 14) - 1);
	// The last 2 bits are used to store flags
	OutFlags = AlphaWithFlags & 0x3;
}

// It's a bit overkill but we have 2 channels available for weight, so we can use 16 bits for storing it
// Weight is packed from a [0, 2^16-1] value to two bytes, each of which we need to provide as a 0-1 float for writing 
float2 PackWeight(float InWeight)
{
	uint PackedWeight = (uint) clamp(round(InWeight * 65535.0f), 0.0f, 65535.0f);
	return float2((float)(PackedWeight >> 8) / 255.0f, (float)(PackedWeight & 0xFF) / 255.0f);
}

float UnpackWeight(float2 InPackedWeight)
{
	return ((InPackedWeight.r * 256 + InPackedWeight.g) * 255) / 65535.0f;
}

// Weight Alpha is 14 bits-wide + 2 bits are left for flags. InWeight is expected in the [0,1] range
float2 PackWeightAlpha(float InAlpha, uint InAlphaFlags)
{
	uint PackedAlpha = (uint)clamp(InAlpha * 65535.0f, 0.0f, 65535.0f); // First 14 bits are used to store alpha (0xFFFC in float)
	uint PackedFlags = InAlphaFlags & 0x3; // Last 2 bits are used to store flags
	return float2((float)(PackedAlpha >> 8) / 255.0, (float)((PackedAlpha & 0xFC) | PackedFlags) / 255.0);
}

void UnpackWeightAlpha(float2 InPackedAlpha, out float OutAlpha, out uint OutFlags)
{
	uint AlphaWithFlags = ((uint)round(InPackedAlpha.r * 255.0f) << 8) | (uint)round(InPackedAlpha.g * 255.0f);
	// The first 14 bits are used to store alpha
	uint ExtractedAlpha = (AlphaWithFlags >> 2);
	OutAlpha = (float)ExtractedAlpha / (float)((1 << 14) - 1);
	// The last 2 bits are used to store flags
	OutFlags = AlphaWithFlags & 0x3;
}

// Compute the normal of the triangle formed by the 3 points (in winding order).
// .xyz of each point is its world space position
// Additionally, the .w component of each point indicates its validity (valid: .w == 1, invalid: .w == 0) 
// the resulting normal will be 0 if any of the points is invalid
float3 ComputeNullableTriangleNormal(float4 InPoint0, float4 InPoint1, float4 InPoint2)
{
	float3 Normal = normalize(cross(InPoint0.xyz - InPoint1.xyz, InPoint1.xyz - InPoint2.xyz));
	return Normal * InPoint0.w * InPoint1.w * InPoint2.w;
}
