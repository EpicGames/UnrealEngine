// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Nanite/NaniteHZBCull.ush"

#ifndef LANDSCAPE_TILE_QUADS
	#define LANDSCAPE_TILE_QUADS 4u
#endif

struct FLandscapeView
{
	float4x4 ViewToClip;
	float4x4 TranslatedWorldToClip;
	float3 RelativePreViewTranslation;
	float _Pad0;
	float3 ViewTilePosition;
	float _Pad1;
};

struct FLandscapeSection
{
	float4x4 LocalToRelativeWorld;
	float3 TilePosition;
	float LocalZ;
	float HalfHeight;
	float NeighborLODExtent;
	float _Pad[2];
};

StructuredBuffer<FLandscapeView> LandscapeViews;
StructuredBuffer<FLandscapeSection> LandscapeSections;

uint NumLandscapeSections;
uint NumLandscapeViews;
uint SubsectionSizeTiles;
uint NumSubsections;
uint NearClip;

RWBuffer<uint> IndirectArgsBufferOut;
RWByteAddressBuffer TileDataOut;

// XY for each tile in a section
// Z for each NumLandscapeSections * NumLandscapeViews 
[numthreads(8, 8, 1)]
void BuildLandscapeTileDataCS(uint3 ThreadID : SV_DispatchThreadID) 
{
	uint SectionSizeTiles = SubsectionSizeTiles * NumSubsections;
	if (ThreadID.x >= SectionSizeTiles)
	{
		return;
	}

	uint ViewIdx = ThreadID.z / NumLandscapeSections;
	uint SectionIdx = ThreadID.z % NumLandscapeSections;
	
	FLandscapeSection Section = LandscapeSections[SectionIdx];
	FLandscapeView LandscapeView = LandscapeViews[ViewIdx];

	const bool bIsOrtho = IsOrthoProjection(LandscapeView.ViewToClip);
	const bool bNearClip = (NearClip != 0u);
	const bool bSkipFrustumCull = false;

	float2 TileExtentQuads = LANDSCAPE_TILE_QUADS * 0.5f;
	float2 TileCenterQuads = (ThreadID.xy * LANDSCAPE_TILE_QUADS) + TileExtentQuads;
	// Inflate tile size to account for neighbors section LOD
	TileExtentQuads += float2(Section.NeighborLODExtent, Section.NeighborLODExtent);

	float3 BoxExtentLocal = float3(TileExtentQuads, Section.HalfHeight);
	float3 BoxCenterLocal = float3(TileCenterQuads, Section.LocalZ);
	
	FLWCMatrix LocalToWorld	= MakeLWCMatrix4x3(Section.TilePosition, Section.LocalToRelativeWorld);
	float4x4 LocalToTranslatedWorld = LWCMultiplyTranslation(LocalToWorld, MakeLWCVector3(-LandscapeView.ViewTilePosition, LandscapeView.RelativePreViewTranslation));
	
	FFrustumCullData CullData = BoxCullFrustum(BoxCenterLocal, BoxExtentLocal, LocalToTranslatedWorld, LandscapeView.TranslatedWorldToClip, LandscapeView.ViewToClip, bIsOrtho, bNearClip, bSkipFrustumCull);
	if (CullData.bIsVisible)
	{
		uint TileOutputIndex = 0u;
		InterlockedAdd(IndirectArgsBufferOut[ThreadID.z * INDIRECT_ARGS_NUM_WORDS + 1u], 1u, TileOutputIndex);
		{
			uint SubsectionX = (ThreadID.x < SubsectionSizeTiles ? 0u : 1u);
			uint SubsectionY = (ThreadID.y < SubsectionSizeTiles ? 0u : 1u);
			uint QuadX = (ThreadID.x - SubsectionSizeTiles * SubsectionX) * LANDSCAPE_TILE_QUADS;
			uint QuadY = (ThreadID.y - SubsectionSizeTiles * SubsectionY) * LANDSCAPE_TILE_QUADS;
			
			uint TileData4Bytes = (SubsectionY << 24u) | (SubsectionX << 16u) | (QuadY << 8u) | (QuadX << 0u);
			uint TileGlobalIndex = TileOutputIndex + (ThreadID.z * SectionSizeTiles * SectionSizeTiles);

			TileDataOut.Store(TileGlobalIndex * 4u, TileData4Bytes);
		}
	}
}
