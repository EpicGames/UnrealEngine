// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "LandscapeCommon.ush"

// ----------------------------------------------------------------------------------

#if defined (__INTELLISENSE__)
// Uncomment the appropriate define for enabling syntax highlighting with HLSL Tools for Visual Studio : 
//#define MERGE_EDIT_LAYER 1
//#define GENERATE_NORMALS 1
//#define GENERATE_MIPS 1
#endif // defined (__INTELLISENSE__)


// ----------------------------------------------------------------------------------

#if MERGE_EDIT_LAYER
// MergeEditLayerPS inputs/outputs :

/** EHeightmapBlendMode enum */
#define EHEIGHTMAPBLENDMODE_ADDITIVE 0
#define EHEIGHTMAPBLENDMODE_LEGACYALPHABLEND 1
#define EHEIGHTMAPBLENDMODE_ALPHABLEND 2

uint InEditLayerBlendMode; // See EHeightmapBlendMode
float InEditLayerAlpha; // Alpha value to be used in the blend

Texture2D<float4> InCurrentEditLayerHeightmap; // The height of the current edit layer to merge 
Texture2D<float4> InPreviousEditLayersHeightmap; // The resulting height from the merge of all prior edit layers
#endif // MERGE_EDIT_LAYER

#if GENERATE_NORMALS
// GenerateNormalsPS inputs/outputs :

uint4 InTextureSize; // .xy == Effective size in pixels of InSourceHeightmap, .zw == actual texture size in pixels of InSourceHeightmap
float3 InLandscapeGridScale; // x == LS Actor DrawScale.X, y == LS Actor DrawScale.y, z == LS Actor DrawScale.z / 128.0f (ZSCALE)
uint InComponentSizeQuads; // Number of quads per component
uint2 InNumComponents; // Number of components in the effective region of the texture (.xy)

// TODO [jonathan.bard] : this can be replaced by GlobalPointClampedSampler :
SamplerState InSourceHeightmapSampler;
Texture2D<float4> InSourceHeightmap;
Texture2D<float> InComponentIdTexture;
#endif // GENERATE_NORMALS

#if GENERATE_MIPS
// GenerateMipsPS inputs/outputs :

uint2 InCurrentMipSubsectionSize; // Size of the the subsection at the currently generated mip level

Texture2D<float4> InSourceHeightmap; // Source heightmap (containing the current mip level - 1)
#endif // GENERATE_MIPS


// ----------------------------------------------------------------------------------
// Util functions : 

#if MERGE_EDIT_LAYER
float ExtractAlpha(float2 InPackedAlpha)
{
	int AlphaWithFlags = ((int) round(InPackedAlpha.r * 255.0f) << 8) | (int) round(InPackedAlpha.g * 255.0f);
	int Alpha = AlphaWithFlags & 0xFFFC; // Last 2 bits are used to store lower/raise flags
	return float(Alpha) / 65532.0f; // 0xFFFC in float
}
#endif // MERGE_EDIT_LAYER

#if GENERATE_NORMALS
float ScaleHeight(float InHeight)
{
	return (InHeight - LANDSCAPE_MID_VALUE) * InLandscapeGridScale.z * TERRAIN_ZSCALE;
}

float ExtractAndScaleHeight(float2 InPackedHeight)
{
	return ScaleHeight(UnpackHeight(InPackedHeight));
}

float UnscaleHeight(float InScaledHeight)
{
	return (InScaledHeight / InLandscapeGridScale.z / TERRAIN_ZSCALE) + LANDSCAPE_MID_VALUE;
}

float4 ExtractSampleInfo(int2 InTextureCoordinates, float2 InPackedHeight)
{
	// Note: don't use a bool here, as some compilers don't support it : 
	int bIsValidSample = (int)(all(InTextureCoordinates >= 0) && all(InTextureCoordinates < (int2)InTextureSize.xy));
	uint ComponentId = asuint(InComponentIdTexture.Load(int3(InTextureCoordinates, 0)).x);
	bIsValidSample &= (ComponentId != INDEX_NONE);
	return float4((InTextureCoordinates + 0.5f) * InLandscapeGridScale.xy, ExtractAndScaleHeight(InPackedHeight), bIsValidSample);
}
#endif // GENERATE_NORMALS


// ----------------------------------------------------------------------------------
// Pixel shaders : 

#if MERGE_EDIT_LAYER
void MergeEditLayerPS(in float4 SVPos : SV_POSITION, out float4 OutPackedHeight : SV_Target0)
{
	uint2 TextureCoordinates = floor(SVPos.xy);
		
	float4 CurrentLayerSample = InCurrentEditLayerHeightmap.Load(int3(TextureCoordinates, 0));
	float CurrentLayerHeight = UnpackHeight(CurrentLayerSample.xy) - LANDSCAPE_MID_VALUE; // xy = relative coordinates, z = mip level
	float4 PreviousLayersSample = InPreviousEditLayersHeightmap.Load(int3(TextureCoordinates, 0));
	float PreviousLayersHeight = UnpackHeight(PreviousLayersSample.xy) - LANDSCAPE_MID_VALUE; // xy = relative coordinates, z = mip level
	float FinalHeight = PreviousLayersHeight;
	
	if (InEditLayerBlendMode == EHEIGHTMAPBLENDMODE_ADDITIVE)
	{
		FinalHeight += InEditLayerAlpha * CurrentLayerHeight;
	}
	else if (InEditLayerBlendMode == EHEIGHTMAPBLENDMODE_LEGACYALPHABLEND)
	{
		float LayerHeightAlphaBlend = ExtractAlpha(CurrentLayerSample.ba);
		float NewHeight = InEditLayerAlpha * CurrentLayerHeight + (LayerHeightAlphaBlend * PreviousLayersHeight);
		uint RaiseLower = floor(CurrentLayerSample.a * 255.f + 0.5f);
		bool bLowerTerrain = (RaiseLower & 1);
		bool bRaiseTerrain = (RaiseLower & 2);
		if ((bRaiseTerrain && NewHeight > PreviousLayersHeight) || (bLowerTerrain && NewHeight < PreviousLayersHeight))
		{
			FinalHeight = NewHeight;
		}
		else
		{
			FinalHeight = PreviousLayersHeight;
		}
	}
	else if (InEditLayerBlendMode == EHEIGHTMAPBLENDMODE_ALPHABLEND)
	{
		float LayerHeightAlphaBlend = 0.0f;
		uint HeightFlags = 0;
		UnpackHeightAlpha(CurrentLayerSample.ba, LayerHeightAlphaBlend, HeightFlags);
		float FinalAlpha = InEditLayerAlpha * LayerHeightAlphaBlend;
		if (HeightFlags == EHEIGHTMAPALPHAFLAGS_ALPHABLEND)
		{
			FinalHeight = lerp(PreviousLayersHeight, CurrentLayerHeight, FinalAlpha);
		}
		else if (HeightFlags & EHEIGHTMAPALPHAFLAGS_MIN)
		{
			FinalHeight = lerp(PreviousLayersHeight, min(PreviousLayersHeight, CurrentLayerHeight), FinalAlpha);
		}
		else if (HeightFlags & EHEIGHTMAPALPHAFLAGS_MAX)
		{
			FinalHeight = lerp(PreviousLayersHeight, max(PreviousLayersHeight, CurrentLayerHeight), FinalAlpha);
		}
		// Additive case (== no flag) :
		else if (HeightFlags == EHEIGHTMAPALPHAFLAGS_ADDITIVE)
		{
			FinalHeight += CurrentLayerHeight * FinalAlpha;
		}
	}
	
	FinalHeight += LANDSCAPE_MID_VALUE;
	FinalHeight = clamp(FinalHeight, 0.0f, LANDSCAPE_MAX_VALUE);

	OutPackedHeight = float4(PackHeight(FinalHeight), 0.0f, 0.0f);
}
#endif // MERGE_EDIT_LAYER


// ----------------------------------------------------------------------------------

#if GENERATE_NORMALS
void GenerateNormalsPS(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	uint2 TextureCoordinates = floor(SVPos.xy);
	uint2 TextureEffectiveSize = InTextureSize.xy;
	uint2 TextureActualSize = InTextureSize.zw;
	float2 TexelSize = 1.0f / TextureActualSize;

	//float MaskMinBorderX = (TextureCoordinates.x == 0) ? 0.0f : 1.0f; // first pixel X
	//float MaskMinBorderY = (TextureCoordinates.y == 0) ? 0.0f : 1.0f; // first pixel Y
	//float MaskMaxBorderX = (TextureCoordinates.x >= TextureEffectiveSize.x - 1) ? 0.0f : 1.0f; // last pixel X
	//float MaskMaxBorderY = (TextureCoordinates.y >= TextureEffectiveSize.y - 1) ? 0.0f : 1.0f; // last pixel Y
	//

	// The triangle topology is the following (where C = center, T = top, B = bottom, L = left, R = right and Nx the normals we need to interpolate):
	// TL ------ TT
	// | \       | \
	// |  \      |  \
	// |   \     |   \
	// | N0 \ N1 | N3 \
	// |     \   |     \
	// |      \  |      \
	// |       \ |       \
	// LL ------ CC ------ RR
	//   \       | \       |
	//    \      |  \      |
	//     \     |   \     |
	//      \ N2 | N4 \ N5 |
	//       \   |     \   |
	//        \  |      \  |
	//         \ |       \ |
	//           BB ------ BR
	// So we can retrieve all 7 samples using 4 gathers : 
	// Make sure we gather the 4 pixels in the bottom right direction by sampling at (0.75, 0.75) of the pixel (gather retrieves the 4 samples that would be used for bilinear interpolation):

	float2 GatherLocation = ((float2) TextureCoordinates + 0.75f) * TexelSize;
	float4 Red0 = InSourceHeightmap.GatherRed(InSourceHeightmapSampler, GatherLocation - TexelSize);
	float4 Green0 = InSourceHeightmap.GatherGreen(InSourceHeightmapSampler, GatherLocation - TexelSize);
	float4 Red1 = InSourceHeightmap.GatherRed(InSourceHeightmapSampler, GatherLocation);
	float4 Green1 = InSourceHeightmap.GatherGreen(InSourceHeightmapSampler, GatherLocation);

	// [x,y,z] = World Position [w] = valid mask
	float4 TL = ExtractSampleInfo(TextureCoordinates + int2(-1, -1), float2(Red0.w, Green0.w));
	float4 TT = ExtractSampleInfo(TextureCoordinates + int2(+0, -1), float2(Red0.z, Green0.z));
	float4 CC = ExtractSampleInfo(TextureCoordinates + int2(+0, +0), float2(Red0.y, Green0.y));
	float4 LL = ExtractSampleInfo(TextureCoordinates + int2(-1, +0), float2(Red0.x, Green0.x));
	float4 RR = ExtractSampleInfo(TextureCoordinates + int2(+1, +0), float2(Red1.z, Green1.z));
	float4 BR = ExtractSampleInfo(TextureCoordinates + int2(+1, +1), float2(Red1.y, Green1.y));
	float4 BB = ExtractSampleInfo(TextureCoordinates + int2(+0, +1), float2(Red1.x, Green1.x));

	//// mask out samples that are off the edge of the texture
	//TL.w *= MaskMinBorderX * MaskMinBorderY;
	//TT.w *= MaskMinBorderY;
	//// CC should never be off the edge
	//LL.w *= MaskMinBorderX;
	//RR.w *= MaskMaxBorderX;
	//BR.w *= MaskMaxBorderX * MaskMaxBorderY;
	//BB.w *= MaskMaxBorderY;

	float3 N0 = ComputeNullableTriangleNormal(CC, LL, TL);
	float3 N1 = ComputeNullableTriangleNormal(TL, TT, CC);
	float3 N2 = ComputeNullableTriangleNormal(LL, CC, BB);
	float3 N3 = ComputeNullableTriangleNormal(RR, CC, TT);
	float3 N4 = ComputeNullableTriangleNormal(BR, BB, CC);
	float3 N5 = ComputeNullableTriangleNormal(CC, RR, BR);

	// average the normals
	float3 FinalNormal = normalize(N0 + N1 + N2 + N3 + N4 + N5);

	// Scale back to be 0-1 normal
	FinalNormal = (FinalNormal + 1.0) * 0.5;
	
	float FinalHeight = UnscaleHeight(CC.z);
	OutColor = float4(PackHeight(FinalHeight), FinalNormal.xy);
}
#endif // GENERATE_NORMALS


// ----------------------------------------------------------------------------------
#if GENERATE_MIPS
void GenerateMipsPS(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	uint2 TextureCoordinates = floor(SVPos.xy);
	
	float4 SourceSamples[4] =
	{
		InSourceHeightmap.Load(int3(2 * TextureCoordinates + int2(+0, +0), 0)),
		InSourceHeightmap.Load(int3(2 * TextureCoordinates + int2(+1, +0), 0)),
		InSourceHeightmap.Load(int3(2 * TextureCoordinates + int2(+0, +1), 0)),
		InSourceHeightmap.Load(int3(2 * TextureCoordinates + int2(+1, +1), 0)),
	};
		
	// Because the borders of each landscape subsection are shared between neighbors, we must ensure that the parent mip's inner row/column of pixels don't contribute, 
	//  so that pixels on the subsection borders for neighboring subsections for mips have an equal value : 
	//  9 possible cases (only the samples marked with a * must be kept): 
	//      bIsMinBorder.x = true
	//     |                        bIsMaxBorder.x = true
	//     |                       |
	//     v                       v
	// +-------+   +-------+   +-------+
	// | * :   |   | * : * |   |   : * |
	// | - + - |...| - + - |...| - + - | <-- bIsMinBorder.y = true
	// |   :   |   |   :   |   |   :   |
	// +-------+   +-------+   +-------+
	//    ...         ...	      ...
	// +-------+   +-------+   +-------+
	// | * :   |   | * : * |   |   : * |
	// | - + - |...| - + - |...| - + - |
	// | * :   |   | * : * |   |   : * |
	// +-------+   +-------+   +-------+
	//    ...         ...	      ...
	// +-------+   +-------+   +-------+
	// |   :   |   |   :   |   |   :   |
	// | - + - |...| - + - |...| - + - | <-- bIsMaxBorder.y = true
	// | * :   |   | * : * |   |   : * | 
	// +-------+   +-------+   +-------+

	bool bIsLastMip = all(InCurrentMipSubsectionSize == 1);
	
	uint2 SubsectionRelativeTextureCoordinates = TextureCoordinates % InCurrentMipSubsectionSize;
	bool2 bIsMinBorder = (SubsectionRelativeTextureCoordinates == 0);
	bool2 bIsMaxBorder = (SubsectionRelativeTextureCoordinates == (InCurrentMipSubsectionSize - 1));

	float SampleWeights[4] =
	{ 
		// On the last mip, it's ok to keep all 4 samples : all neighbors components share them : 
		((bIsMaxBorder.x || bIsMaxBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMinBorder.x || bIsMaxBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMaxBorder.x || bIsMinBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMinBorder.x || bIsMinBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
	};
	
	float TotalSampleWeight = 0.0f;
	OutColor = 0.0f;
	
	float3 HeightAndNormal = 0.0f;
	
	UNROLL
	for (int i = 0; i < 4; ++i)
	{
		HeightAndNormal += float3(UnpackHeight(SourceSamples[i].xy), SourceSamples[i].zw) * SampleWeights[i];
		TotalSampleWeight += SampleWeights[i];
	}
	HeightAndNormal /= TotalSampleWeight;
	
	OutColor.xy = PackHeight(HeightAndNormal.x);
	OutColor.zw = HeightAndNormal.yz;
}
#endif // GENERATE_MIPS