// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "LandscapeCommon.ush"

// ----------------------------------------------------------------------------------

#if defined (__INTELLISENSE__)
// Uncomment the appropriate define for enabling syntax highlighting with HLSL Tools for Visual Studio : 
//#define RENDER_COMPONENTID 1
//#define COPY_QUADS 1
//#define COPY_QUADS_MULTISOURCE 1
//#define COPY_WEIGHTMAP 1
//#define RESOLVE_LAYER_DATA 1
#endif // defined (__INTELLISENSE__)


// ----------------------------------------------------------------------------------

#if RENDER_COMPONENTID
// RenderComponentIdPS inputs/outputs :
Buffer<uint> InQuadComponentIds; // Id to write into the output for each quad 
#endif // RENDER_COMPONENTID

#if COPY_QUADS
// CopyQuadsPS inputs/outputs :

Texture2D<float4> InSourceTexture;
#endif // COPY_QUADS

#if COPY_QUADS_MULTISOURCE
// CopyQuadsMultiSourcePS inputs/outputs :

// Helper macro for handling an array of N textures (currently using 63 because 64 is the maximum supported number of SRV on all platforms needing to run this) :
#define FOREACH_SOURCETEXTURE(op) \
	op(0) \
	op(1) \
	op(2) \
	op(3) \
	op(4) \
	op(5) \
	op(6) \
	op(7) \
	op(8) \
	op(9) \
	op(10) \
	op(11) \
	op(12) \
	op(13) \
	op(14) \
	op(15) \
	op(16) \
	op(17) \
	op(18) \
	op(19) \
	op(20) \
	op(21) \
	op(22) \
	op(23) \
	op(24) \
	op(25) \
	op(26) \
	op(27) \
	op(28) \
	op(29) \
	op(30) \
	op(31) \
	op(32) \
	op(33) \
	op(34) \
	op(35) \
	op(36) \
	op(37) \
	op(38) \
	op(39) \
	op(40) \
	op(41) \
	op(42) \
	op(43) \
	op(44) \
	op(45) \
	op(46) \
	op(47) \
	op(48) \
	op(49) \
	op(50) \
	op(51) \
	op(52) \
	op(53) \
	op(54) \
	op(55) \
	op(56) \
	op(57) \
	op(58) \
	op(59) \
	op(60) \
	op(61) 

Buffer<uint4> InQuadInfos; // .x = source texture index, .y = source texture channel (for weightmaps), .z = component id of the component this quad corresponds to, .w = unused
Texture2D<float> InComponentIdTexture; // texture containing the linear index of the components in this batch. This allows to resolve conflicts in case multiple quads render to the same pixel

// Define N input textures :
#define DECLARE_SOURCE_TEXTURE(n) Texture2D<float4> InSourceTexture_##n;
FOREACH_SOURCETEXTURE(DECLARE_SOURCE_TEXTURE)
#undef DECLARE_SOURCE_TEXTURE
#endif // COPY_QUADS_MULTISOURCE

#if RESOLVE_LAYER_DATA
Texture2D<float4> InSourceTexture;
#endif // RESOLVE_LAYER_DATA

// ----------------------------------------------------------------------------------
// Pixel shaders : 

#if RENDER_COMPONENTID

void RenderComponentIdPS(float4 SvPosition : SV_POSITION, float2 SourceUV : TEXCOORD0, float2 UnusedQuadUV : TEXCOORD1, float InRectIndex : RECT_INDEX, out float OutColor : SV_Target0)
{
	uint QuadIndex = (uint)InRectIndex;
	OutColor = asfloat(InQuadComponentIds[QuadIndex]);
}

#endif // RENDER_COMPONENTID


// ----------------------------------------------------------------------------------

#if COPY_QUADS

void CopyQuadsPS(float4 SvPosition : SV_POSITION, float2 QuadUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = InSourceTexture.SampleLevel(GlobalPointClampedSampler, QuadUV, 0);
}

#endif // COPY_QUADS


// ----------------------------------------------------------------------------------

#if COPY_QUADS_MULTISOURCE

float4 GetSourceTextureColor(uint SourceTextureIndex, float2 SourceUV)
{
	// Having a switch with returns is necessaire in Vulkan (see  UE-369443):
	// We have an issue in Vulkan when the switch is too large, the large if-else causes issues under investigation.

	BRANCH
	switch (SourceTextureIndex)
	{
	#define IMPLEMENT_SAMPLE_SWITCH_CASE(n) case n: return InSourceTexture_##n.SampleLevel(GlobalPointClampedSampler, SourceUV, 0);
		FOREACH_SOURCETEXTURE(IMPLEMENT_SAMPLE_SWITCH_CASE);
	#undef IMPLEMENT_SAMPLE_SWITCH_CASE

	default: return float4(0.0f, 0.0f, 0.0f, 0.0f);
	}
}

void CopyQuadsMultiSourcePS(float4 SvPosition : SV_POSITION, float2 SourceUV : TEXCOORD0, float2 UnusedQuadUV : TEXCOORD1, float InRectIndex : RECT_INDEX, out float4 OutColor : SV_Target0)
{
	uint QuadIndex = (uint)InRectIndex;
	uint SourceTextureIndex = InQuadInfos[QuadIndex].x;
	uint QuadComponentId = InQuadInfos[QuadIndex].z;

	float ComponentIdSample = InComponentIdTexture.Load(int3(SvPosition.xy, 0)); // xy = coordinates, z = mip level
	uint ComponentIdUint = asuint(ComponentIdSample);
	// if we're not the authoritative component for this pixel, bail out and don't render anything :
	if (ComponentIdUint != QuadComponentId)
	{
		discard;
	}

	const float4 ColorSample = GetSourceTextureColor(SourceTextureIndex, SourceUV);

	OutColor = 0.0f;

#if COPY_WEIGHTMAP
	// For weightmaps, copy from the appropriate channel and pack : 
	uint SourceChannelIndex = InQuadInfos[QuadIndex].y;
	float Weight = ColorSample[SourceChannelIndex];
	OutColor = float4(PackWeight(Weight), PackWeightAlpha(1.0f, EWEIGHTMAPALPHAFLAGS_NONE));
#else // COPY_WEIGHTMAP
	OutColor = ColorSample;
#endif // !COPY_WEIGHTMAP
}

#endif // COPY_QUADS_MULTISOURCE

// ----------------------------------------------------------------------------------

#if RESOLVE_LAYER_DATA

void ResolveLayerData(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	float4 ColorSample = InSourceTexture.Load(int3(SVPos.xy, 0)); // xy = coordinates, z = mip level

	// TODO [jonathan.bard] : since this step is not a plain copy anymore, we should be able to support different format types (e.g. instead 
	//  of expecting heightmaps to be compressed, we could read it in a different way) and support different input resolutions (e.g. BP brushes 
	//  wouldn't have the necessity to use the same resolution as the merge RT). We could also support an alpha channel and/or other alpha flags : 
#if RESOLVE_WEIGHTMAP
	float Weight = ColorSample.x;
	OutColor = float4(PackWeight(Weight), PackWeightAlpha(1.0f, EWEIGHTMAPALPHAFLAGS_NONE));
#else // RESOLVE_WEIGHTMAP
	// Currently, it's just a plain copy for heightmaps : 
	OutColor = ColorSample;
#endif // !RESOLVE_WEIGHTMAP
}

#endif // RESOLVE_LAYER_DATA