// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "LandscapeCommon.ush"

// ----------------------------------------------------------------------------------

#if defined (__INTELLISENSE__)
// Uncomment the appropriate define for enabling syntax highlighting with HLSL Tools for Visual Studio : 
//#define MERGE_EDIT_LAYER 1
//#define PERFORM_FINAL_WEIGHT_BLENDING 1
//#define PACK_WEIGHTMAP 1
//#define GENERATE_MIPS 1
#endif // defined (__INTELLISENSE__)


// ----------------------------------------------------------------------------------

/** EWeightmapBlendMode enum */
#define EWEIGHTMAPBLENDMODE_ADDITIVE 0
#define EWEIGHTMAPBLENDMODE_SUBTRACTIVE 1
#define EWEIGHTMAPBLENDMODE_PASSTHROUGH 2
#define EWEIGHTMAPBLENDMODE_ALPHABLEND 3

/** EWeightmapTargetLayerFlags enum */
#define EWEIGHTMAPTARGETLAYERFLAGS_ISVISIBILITYLAYER (1 << 0)
// EWEIGHTMAPTARGETLAYERFLAGS_ISWEIGHTBLENDED is used for "final weight blending", whose formula is : 
//  * BlendedWeightsSum = weight sum of all final weight-blended target layers
//  * LayerWeight = weight of the target layer being processed
//  ==> FinalLayerWeight = LayerWeight / BlendedWeightsSum
#define EWEIGHTMAPTARGETLAYERFLAGS_ISWEIGHTBLENDED (1 << 1)
#define EWEIGHTMAPTARGETLAYERFLAGS_SKIP (1 << 2)
// EWEIGHTMAPTARGETLAYERFLAGS_ISPREMULTIPLIEDALPHAWEIGHTBLENDED is used for "premultiplied alpha blending", whose formula is : 
//  * PreviousLayersWeight = weight of the target layer being processed, at this step of the edit layers merge algorithm (i.e. merged weight of all previous edit layers in the stack)
//  * CurrentLayerWeight = weight of the target layer being processed at the current edit layer 
//  * PremultipliedAlphaBlendGroupWeightSum = weight sum of all premultiplied alpha-blended target layers of the same blend group at the current edit layer 
//  * EditLayerAlpha = alpha value of the current edit layer
// => FinalLayerWeight = PreviousLayersWeight * (1.0f - min(PremultipliedAlphaBlendGroupWeightSum, 1.0f)) 
//     + EditLayerAlpha * CurrentLayerWeight / max(PremultipliedAlphaBlendGroupWeightSum, 1.0f);
#define EWEIGHTMAPTARGETLAYERFLAGS_ISPREMULTIPLIEDALPHAWEIGHTBLENDED (1 << 3)

#if MERGE_EDIT_LAYER
// MergeEditLayerPS inputs/outputs :

// Per-target layer information : 
struct FMergeEditLayerTargetLayerInfo
{
	uint Flags; // See EWeightmapTargetLayerFlags
	int BlendGroupIndex; // Defines the target layer blend group that this target layer belongs to in this blend operation. -1 if not applicable
};

uint InTargetLayerIndex; // Index of the target layer being processed
uint InNumTargetLayers; // Number of target layers (i.e. == InMergeEditLayerTargetLayerInfos.Num())
uint InEditLayerTargetLayerBlendMode; // See EWeightmapBlendMode. Target layer's blend mode, which is per-edit layer and per-target layer and can therefore differ from one edit layer to another
float InEditLayerAlpha; // Global alpha value of the edit layer currently being merged

// Array that contains per-target layer information (e.g. target layer's flags)
StructuredBuffer<FMergeEditLayerTargetLayerInfo> InMergeEditLayerTargetLayerInfos;
Texture2DArray<float4> InCurrentEditLayerWeightmaps; // The target layers of the current edit layer to merge (1 slice per weightmap in the target layer group)
Texture2DArray<float4> InPreviousEditLayersWeightmaps; // The result from the merge of all prior edit layers (1 slice per weightmap in the target layer group)
// Array that contains per-edit layer / per-target layer information (e.g. the target layer's blend mode, which is per-edit layer and can therefore differ from one edit layer to another)
#endif // MERGE_EDIT_LAYER

#if PERFORM_FINAL_WEIGHT_BLENDING
// PerformFinalWeightBlendingPS inputs/outputs :

// Per-target layer information : 
struct FFinalWeightBlendingTargetLayerInfo
{
	uint Flags; // See EWeightmapTargetLayerFlags
};

uint InTargetLayerIndex; // Index of the target layer being processed
uint InNumTargetLayers; // Number of target layers (i.e. == InFinalWeightBlendingTargetLayerInfos.Num())
// Array that contains per-target layer information (e.g. target layer's flags)
StructuredBuffer<FFinalWeightBlendingTargetLayerInfo> InFinalWeightBlendingTargetLayerInfos;
Texture2DArray<float4> InCurrentEditLayerWeightmaps; // The texture to horizontally blend (1 slice per weightmap in the target layer group)
#endif // PERFORM_FINAL_WEIGHT_BLENDING

#if PACK_WEIGHTMAP
// PackWeightmapPS inputs/outputs :

int4 InSourceSliceIndices; // For each channel of the output texture (rgba), this indicates the slice index in InSourceWeightmaps where to read from
uint4 InSourcePixelOffsets[4];  // For each channel, offset to add to the pixel's coordinates to load the proper sample in InSourceWeightmaps (.xy, .zw is unused, only there for alignment purposes)
uint2 InSubsectionPixelOffset; // Offset of the subsection currently being rendered
uint InIsAdditive; // = 1 if some channels of this texture have been packed in a previous draw, 0 otherwise

Texture2DArray<float4> InSourceWeightmaps; // Source, single-channel (we don't care about the alpha flags at this point), texture to pack
Texture2D<float4> InWeightmapBeingPacked; // The weightmap being packed, in case the packing operation is additive, i.e. it occurs across multiple draws (contains the channels that have previously been packed)
#endif // PACK_WEIGHTMAP

#if GENERATE_MIPS
// GenerateMipsPS inputs/outputs :

uint2 InCurrentMipSubsectionSize; // Size of the the subsection at the currently generated mip level

Texture2D<float4> InSourceWeightmap; // Source weightmap (containing the current mip level - 1)
#endif // GENERATE_MIPS


// ----------------------------------------------------------------------------------
// Util functions : 

#if MERGE_EDIT_LAYER
bool IsPremultipliedAlphaWeightBlendedTargetLayer(FMergeEditLayerTargetLayerInfo InTargetLayerInfo)
{
	return (InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_ISPREMULTIPLIEDALPHAWEIGHTBLENDED)
		// For the visibility layer, deactivate weight blending altogether : 
		&& ((InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_ISVISIBILITYLAYER) == 0)
		// Some layers are absent from the batch, they should be skipped from blending too :
		&& ((InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_SKIP) == 0); 
}
#endif // MERGE_EDIT_LAYER

#if PERFORM_FINAL_WEIGHT_BLENDING
bool IsWeightBlendedTargetLayer(FFinalWeightBlendingTargetLayerInfo InTargetLayerInfo)
{
	return (InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_ISWEIGHTBLENDED)
		// For the visibility layer, deactivate weight blending altogether : 
		&& ((InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_ISVISIBILITYLAYER) == 0)
		// Some layers are absent from the batch, they should be skipped from blending too :
		&& ((InTargetLayerInfo.Flags & EWEIGHTMAPTARGETLAYERFLAGS_SKIP) == 0); 
}
#endif // PERFORM_FINAL_WEIGHT_BLENDING


// ----------------------------------------------------------------------------------
// Pixel shaders : 

#if MERGE_EDIT_LAYER
float PerformPremultipliedAlphaBlending(float InPreviousLayersWeight, float InCurrentLayerWeight, float InPremultipliedAlphaBlendGroupWeightSum)
{
	return
		// Previous layers contribution, balanced with the relative weight of other blended target layers
		InPreviousLayersWeight * (1.0f - min(InPremultipliedAlphaBlendGroupWeightSum, 1.0f)) 
		// Current layer contribution, balanced with the relative weight of other blended target layers
		+ InEditLayerAlpha * InCurrentLayerWeight / max(InPremultipliedAlphaBlendGroupWeightSum, 1.0f);
}

void MergeEditLayerPS(in float4 SVPos : SV_POSITION, out float4 OutPackedWeight : SV_Target0)
{
	check((InTargetLayerIndex >= 0) && (InTargetLayerIndex < InNumTargetLayers));
	uint2 TextureCoordinates = floor(SVPos.xy);

	float2 PreviousLayersWeightSample = InPreviousEditLayersWeightmaps.Load(int4(TextureCoordinates, InTargetLayerIndex, 0)).xy; // xy = relative coordinates, z = index in texture array, w = mip level
	float PreviousLayersWeight = UnpackWeight(PreviousLayersWeightSample);
	float FinalWeight = PreviousLayersWeight;
	if (InEditLayerTargetLayerBlendMode != EWEIGHTMAPBLENDMODE_PASSTHROUGH)
	{
		float4 CurrentLayerWeightSample = InCurrentEditLayerWeightmaps.Load(int4(TextureCoordinates, InTargetLayerIndex, 0)); // xy = relative coordinates, z = index in texture array, w = mip level
		float CurrentLayerWeight = UnpackWeight(CurrentLayerWeightSample.xy);
		float CurrentLayerWeightAlpha = 1.0f;
		uint CurrentLayerWeightFlags = EWEIGHTMAPALPHAFLAGS_NONE;
		UnpackWeightAlpha(CurrentLayerWeightSample.zw, CurrentLayerWeightAlpha, CurrentLayerWeightFlags);

		// TODO [jonathan.bard] : remove InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_SUBTRACTIVE and add the EWEIGHTMAPALPHAFLAGS_SUBTRACTIVE per-pixel flag ?
		// TODO [jonathan.bard] : handle subtractive for premultiplied alpha (it's yet another "exotic" blend mode)
		if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_SUBTRACTIVE)
		{
			float FinalAlpha = InEditLayerAlpha * CurrentLayerWeightAlpha;
			FinalWeight -= CurrentLayerWeight * FinalAlpha;
		}
		else 
		{
			FMergeEditLayerTargetLayerInfo OutputTargetLayerInfo = InMergeEditLayerTargetLayerInfos[InTargetLayerIndex];
			if (IsPremultipliedAlphaWeightBlendedTargetLayer(OutputTargetLayerInfo))
			{
				// Start again from scratch 
				FinalWeight = 0.0f;
				check(OutputTargetLayerInfo.BlendGroupIndex >= 0);

				// Compute the weight sum of all other target layers belonging to the same blend group 
				float PremultipliedAlphaBlendGroupWeightSum = 0.0f;
				bool bNeedsNonAdditiveBlendStep = false;

				LOOP
				for (uint OtherTargetLayerIndex = 0; OtherTargetLayerIndex < InNumTargetLayers; ++OtherTargetLayerIndex)
				{
					FMergeEditLayerTargetLayerInfo TargetLayerInfo = InMergeEditLayerTargetLayerInfos[OtherTargetLayerIndex];
					if (IsPremultipliedAlphaWeightBlendedTargetLayer(TargetLayerInfo) 
						&& (OutputTargetLayerInfo.BlendGroupIndex == TargetLayerInfo.BlendGroupIndex))
					{
						check(TargetLayerInfo.BlendGroupIndex >= 0);

						bool bIsCurrentLayer = (OtherTargetLayerIndex == InTargetLayerIndex);
						float4 OtherWeightSample = InCurrentEditLayerWeightmaps.Load(int4(TextureCoordinates, OtherTargetLayerIndex, 0)); // xy = relative coordinates, z = index in texture array, w = mip level
						float OtherWeight = UnpackWeight(OtherWeightSample.xy);
						float OtherWeightAlpha = 1.0f;
						uint OtherWeightFlags = EWEIGHTMAPALPHAFLAGS_NONE;
						UnpackWeightAlpha(OtherWeightSample.zw, OtherWeightAlpha, OtherWeightFlags);

						float FinalOtherWeight = 0.0f;
						if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ADDITIVE)
						{
							FinalOtherWeight = OtherWeight * OtherWeightAlpha;
							PremultipliedAlphaBlendGroupWeightSum += FinalOtherWeight * InEditLayerAlpha;
						}
						else if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ALPHABLEND)
						{
							// Additive case (== no flag), we can handle with the other premultiplied alpha layers : 
							if (OtherWeightFlags == EWEIGHTMAPALPHAFLAGS_ADDITIVE)
							{
								FinalOtherWeight = OtherWeight * OtherWeightAlpha;
								PremultipliedAlphaBlendGroupWeightSum += FinalOtherWeight * InEditLayerAlpha;
							}
							// Other, more exotic, blend modes have to be handled in a second phase and then re-balanced with the others accordingly :
							else
							{
								bNeedsNonAdditiveBlendStep = true;
							}
						}
						else
						{
							check(false);
						}

						// Remember the weight of the current layer for the final computation : 
						if (bIsCurrentLayer)
						{
							FinalWeight = FinalOtherWeight;
						}
					}
				}

				// If all layers are premultiplied alpha-blended, perform the final computation already and be done with it : 
				if (!bNeedsNonAdditiveBlendStep)
				{
					FinalWeight = PerformPremultipliedAlphaBlending(PreviousLayersWeight, FinalWeight, PremultipliedAlphaBlendGroupWeightSum);
				}
				// Otherwise (some of the target layers had a non-standard blend), we have to compute each of the layers' final values in order to compute a new sum and adjust the final result accordingly :
				else
				{
					// Start again from scratch : 
					FinalWeight = 0.0;
					float AdjustedBlendGroupWeightSum = 0.0f;
					LOOP
					for (uint OtherTargetLayerIndex = 0; OtherTargetLayerIndex < InNumTargetLayers; ++OtherTargetLayerIndex)
					{
						FMergeEditLayerTargetLayerInfo TargetLayerInfo = InMergeEditLayerTargetLayerInfos[OtherTargetLayerIndex];
						if (IsPremultipliedAlphaWeightBlendedTargetLayer(TargetLayerInfo) 
							&& (OutputTargetLayerInfo.BlendGroupIndex == TargetLayerInfo.BlendGroupIndex))
						{
							check(TargetLayerInfo.BlendGroupIndex >= 0);

							bool bIsCurrentLayer = (OtherTargetLayerIndex == InTargetLayerIndex);
							float4 OtherWeightSample = InCurrentEditLayerWeightmaps.Load(int4(TextureCoordinates, OtherTargetLayerIndex, 0)); // xy = relative coordinates, z = index in texture array, w = mip level
							float OtherWeight = UnpackWeight(OtherWeightSample.xy);
							float OtherWeightAlpha = 1.0f;
							uint OtherWeightFlags = EWEIGHTMAPALPHAFLAGS_NONE;
							UnpackWeightAlpha(OtherWeightSample.zw, OtherWeightAlpha, OtherWeightFlags);

							float4 PreviousOtherWeightSample = InPreviousEditLayersWeightmaps.Load(int4(TextureCoordinates, OtherTargetLayerIndex, 0)); // xy = relative coordinates, z = index in texture array, w = mip level
							float PreviousOtherWeight = UnpackWeight(PreviousOtherWeightSample.xy);

							float FinalOtherWeight = 0.0f;
							if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ADDITIVE)
							{
								// Compute the final weight value of this layer here and add it to the final sum so that we can re-adjust it against the other non-additive cases at the end :
								// Compute (again) the layer's additive contribution
								FinalOtherWeight = OtherWeight * OtherWeightAlpha;
								// Then perform premultiplied alpha blending to get the final layer's contribution, as if all layers were premultiplied alpha blended :
								FinalOtherWeight = PerformPremultipliedAlphaBlending(PreviousOtherWeight, FinalOtherWeight, PremultipliedAlphaBlendGroupWeightSum);
							}
							else if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ALPHABLEND)
							{
								// Additive case (== no flag) : compute the final weight value of this layer here and add it to the final sum so that we can re-adjust it against the other non-additive cases at the end :
								if (OtherWeightFlags == EWEIGHTMAPALPHAFLAGS_ADDITIVE)
								{
									// Compute the final weight value of this layer here and add it to the final sum so that we can re-adjust it against the other non-additive cases at the end :
									// Compute (again) the layer's additive contribution
									FinalOtherWeight = OtherWeight * OtherWeightAlpha;
									// Then perform premultiplied alpha blending to get the final layer's contribution, as if all layers were premultiplied alpha blended :
									FinalOtherWeight = PerformPremultipliedAlphaBlending(PreviousOtherWeight, FinalOtherWeight, PremultipliedAlphaBlendGroupWeightSum);
								}
								else
								{
									// Handle exotic alpha blend cases (min/max/alphablend) here 
									if (OtherWeightFlags == EWEIGHTMAPALPHAFLAGS_ALPHABLEND)
									{
										FinalOtherWeight = lerp(PreviousOtherWeight, OtherWeight, OtherWeightAlpha);
									}
									else if (OtherWeightFlags & EWEIGHTMAPALPHAFLAGS_MIN)
									{
										FinalOtherWeight = lerp(PreviousOtherWeight, min(PreviousOtherWeight, OtherWeight), OtherWeightAlpha);
									}
									else if (OtherWeightFlags & EWEIGHTMAPALPHAFLAGS_MAX)
									{
										FinalOtherWeight = lerp(PreviousOtherWeight, max(PreviousOtherWeight, OtherWeight), OtherWeightAlpha);
									}
									else
									{
										check(false);
									}
								}
							}
							else
							{
								check(false);
							}
							AdjustedBlendGroupWeightSum += FinalOtherWeight;

							// Remember the weight of the current layer for the final computation : 
							if (bIsCurrentLayer)
							{
								FinalWeight = FinalOtherWeight;
							}
						}
					}

					// Finally, perform a final, standard, weight-blending step to make sure all target layers are balanced and their cumulative weight is <= 1.0 : 
					FinalWeight /= max(AdjustedBlendGroupWeightSum, 1.0f);
				}
			}
			// Standard case : no inter-target layer blending to perform
			else
			{
				float FinalAlpha = InEditLayerAlpha * CurrentLayerWeightAlpha;
				if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ADDITIVE)
				{
					FinalWeight = PreviousLayersWeight + CurrentLayerWeight * FinalAlpha;
				}
				else if (InEditLayerTargetLayerBlendMode == EWEIGHTMAPBLENDMODE_ALPHABLEND)
				{
					if (CurrentLayerWeightFlags == EWEIGHTMAPALPHAFLAGS_ALPHABLEND)
					{
						FinalWeight = lerp(PreviousLayersWeight, CurrentLayerWeight, FinalAlpha);
					}
					else if (CurrentLayerWeightFlags & EWEIGHTMAPALPHAFLAGS_MIN)
					{
						FinalWeight = lerp(PreviousLayersWeight, min(PreviousLayersWeight, CurrentLayerWeight), FinalAlpha);
					}
					else if (CurrentLayerWeightFlags & EWEIGHTMAPALPHAFLAGS_MAX)
					{
						FinalWeight = lerp(PreviousLayersWeight, max(PreviousLayersWeight, CurrentLayerWeight), FinalAlpha);
					}
					// Additive case (== no flag) :
					else if (CurrentLayerWeightFlags == EWEIGHTMAPALPHAFLAGS_ADDITIVE)
					{
						FinalWeight = PreviousLayersWeight + CurrentLayerWeight * FinalAlpha;
					}
				}
				else
				{
					check(false);
				}
			}
		}
	}

	OutPackedWeight = float4(PackWeight(FinalWeight), 0.0f, 0.0f);
}
#endif // MERGE_EDIT_LAYER

#if PERFORM_FINAL_WEIGHT_BLENDING
void PerformFinalWeightBlendingPS(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	check((InTargetLayerIndex >= 0) && (InTargetLayerIndex < InNumTargetLayers));

	uint2 TextureCoordinates = floor(SVPos.xy);
	float FinalWeight = 0.0f;
	
	FFinalWeightBlendingTargetLayerInfo ActiveTargetLayerInfo = InFinalWeightBlendingTargetLayerInfos[InTargetLayerIndex];
	if (!IsWeightBlendedTargetLayer(ActiveTargetLayerInfo))
	{
		FinalWeight = UnpackWeight(InCurrentEditLayerWeightmaps.Load(int4(TextureCoordinates, InTargetLayerIndex, 0)).xy); // xy = relative coordinates, z = index in texture array, w = mip level
	}
	else
	{
		float ActiveTargetLayerWeight = 0.0f;
		float BlendedWeightsSum = 0.0f;

		LOOP
		for (uint i = 0; i < InNumTargetLayers; ++i)
		{
			bool bIsOutputTargetLayer = (i == InTargetLayerIndex);
			FFinalWeightBlendingTargetLayerInfo TargetLayerInfo = InFinalWeightBlendingTargetLayerInfos[i];

			// Only weight blended (and non-visibility) target layers participate to weight blending : 
			if (IsWeightBlendedTargetLayer(TargetLayerInfo))
			{
				float Weight = UnpackWeight(InCurrentEditLayerWeightmaps.Load(int4(TextureCoordinates, i, 0)).xy); // xy = relative coordinates, z = index in texture array, w = mip level
				if (bIsOutputTargetLayer)
				{
					ActiveTargetLayerWeight = Weight;
				}
				BlendedWeightsSum += Weight;
			}
		}

		FinalWeight = (BlendedWeightsSum > 0.0f) ? saturate(ActiveTargetLayerWeight / BlendedWeightsSum) : ActiveTargetLayerWeight;
	}

	OutColor = float4(PackWeight(FinalWeight), 0.0f, 0.0f);
}
#endif // PERFORM_FINAL_WEIGHT_BLENDING

// ----------------------------------------------------------------------------------

#if PACK_WEIGHTMAP
void PackWeightmapPS(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	uint2 LocalCoordinates = floor(SVPos.xy);
	uint2 SubsectionRelativeTextureCoordinates = LocalCoordinates - InSubsectionPixelOffset;
	OutColor = 0.0f;
	
	if (InIsAdditive != 0)
	{
		OutColor = InWeightmapBeingPacked.Load(int3(LocalCoordinates, 0)); // xy = relative coordinates, z = mip level
	}
	
	UNROLL
	for (uint i = 0; i < 4; ++i)
	{
		if (InSourceSliceIndices[i] >= 0)
		{
			uint2 TextureCoordinates = SubsectionRelativeTextureCoordinates + InSourcePixelOffsets[i].xy;
			OutColor[i] = UnpackWeight(InSourceWeightmaps.Load(int4(TextureCoordinates, InSourceSliceIndices[i], 0)).xy); // xy = relative coordinates, z = index in texture array, w = mip level
		}
	}
}
#endif // PACK_WEIGHTMAP


// ----------------------------------------------------------------------------------
#if GENERATE_MIPS
void GenerateMipsPS(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	uint2 TextureCoordinates = floor(SVPos.xy);
	
	float4 SourceSamples[4] =
	{
		InSourceWeightmap.Load(int3(2 * TextureCoordinates + int2(+0, +0), 0)),
		InSourceWeightmap.Load(int3(2 * TextureCoordinates + int2(+1, +0), 0)),
		InSourceWeightmap.Load(int3(2 * TextureCoordinates + int2(+0, +1), 0)),
		InSourceWeightmap.Load(int3(2 * TextureCoordinates + int2(+1, +1), 0)),
	};
		
	// Because the borders of each landscape subsection are shared between neighbors, we must ensure that the parent mip's inner row/column of pixels don't contribute, 
	//  so that pixels on the subsection borders for neighboring subsections for mips have an equal value : 
	//  9 possible cases (only the samples marked with a * must be kept): 
	//      bIsMinBorder.x = true
	//     |                        bIsMaxBorder.x = true
	//     |                       |
	//     v                       v
	// +-------+   +-------+   +-------+
	// | * :   |   | * : * |   |   : * |
	// | - + - |...| - + - |...| - + - | <-- bIsMinBorder.y = true
	// |   :   |   |   :   |   |   :   |
	// +-------+   +-------+   +-------+
	//    ...         ...	      ...
	// +-------+   +-------+   +-------+
	// | * :   |   | * : * |   |   : * |
	// | - + - |...| - + - |...| - + - |
	// | * :   |   | * : * |   |   : * |
	// +-------+   +-------+   +-------+
	//    ...         ...	      ...
	// +-------+   +-------+   +-------+
	// |   :   |   |   :   |   |   :   |
	// | - + - |...| - + - |...| - + - | <-- bIsMaxBorder.y = true
	// | * :   |   | * : * |   |   : * | 
	// +-------+   +-------+   +-------+

	bool bIsLastMip = all(InCurrentMipSubsectionSize == 1);

	uint2 SubsectionRelativeTextureCoordinates = TextureCoordinates % InCurrentMipSubsectionSize;
	bool2 bIsMinBorder = (SubsectionRelativeTextureCoordinates == 0);
	bool2 bIsMaxBorder = (SubsectionRelativeTextureCoordinates == (InCurrentMipSubsectionSize - 1));
		
	float SampleWeights[4] =
	{ 
		// On the last mip, it's ok to keep all 4 samples : all neighbors components share them : 
		((bIsMaxBorder.x || bIsMaxBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMinBorder.x || bIsMaxBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMaxBorder.x || bIsMinBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
		((bIsMinBorder.x || bIsMinBorder.y) && !bIsLastMip) ? 0.0f : 1.0f,
	};
	
	float TotalSampleWeight = 0.0f;
	OutColor = 0.0f;
	
	UNROLL
	for (int i = 0; i < 4; ++i)
	{
		OutColor += SourceSamples[i] * SampleWeights[i];
		TotalSampleWeight += SampleWeights[i];
	}
	OutColor /= TotalSampleWeight;
}
#endif // GENERATE_MIPS