// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "LandscapeCommon.ush"

// ----------------------------------------------------------------------------------

#if defined (__INTELLISENSE__)
// Uncomment the appropriate define for enabling syntax highlighting with HLSL Tools for Visual Studio : 
//#define IS_HEIGHTMAP 1
//#define MERGE_TEXTURE 1
//#define RESAMPLE_MERGED_TEXTURE 1
//#define COMPRESS_HEIGHT 1
//#define PACK_RGBA_CHANNELS 1
#endif // defined (__INTELLISENSE__)

// ----------------------------------------------------------------------------------

#if MERGE_TEXTURE
// MergeTexture inputs/outputs :

uint4 InAtlasSubregion; // Subregion of the atlas texture that is being rendered to
uint4 InSourceTextureSubregion; // Subregion of the input texture that we copy from
int InSourceTextureChannel; // Channel of the input texture that we copy from (for weightmaps only)
Texture2D<float4> InSourceTexture; // Source texture (packed height in the .rg channels for heightmap, weight in one the 4 .rgba channels for the weightmap)
#endif // MERGE_TEXTURE

#if RESAMPLE_MERGED_TEXTURE
// ResampleTexture inputs/outputs :

float4x4 InOutputUVToMergedTextureUV; // Transform to apply to the texture coordinates in order to resample the atlas in the requested render area
Texture2D<float> InMergedTexture; // Source texture (unpacked floating point height for heightmap)
SamplerState InMergedTextureSampler;
uint2 InRenderAreaSize; // Size of the output render target
#endif // RESAMPLE_MERGED_TEXTURE

#if PACK_RGBA_CHANNELS
int InNumChannels; // Number of channels to pack
Texture2D<float> InSourceTextures_0; // Source, single-channel, texture to pack
Texture2D<float> InSourceTextures_1; // Source, single-channel, texture to pack
Texture2D<float> InSourceTextures_2; // Source, single-channel, texture to pack
Texture2D<float> InSourceTextures_3; // Source, single-channel, texture to pack
#endif // PACK_RGBA_CHANNELS


// ----------------------------------------------------------------------------------
// Util functions : 

#if MERGE_TEXTURE

void MergeTexture(in float4 SVPos : SV_POSITION, out float OutValue : SV_Target0)
{
	// Input texture and output texture atlas may not be at the same resolution (different mips), so we need to renormalize the sample position before sampling the input texture :
	uint2 AtlasSubregionSize = InAtlasSubregion.zw - InAtlasSubregion.xy;
	float2 SampleCoordinatesNormalized = (SVPos.xy - InAtlasSubregion.xy) / (float2) AtlasSubregionSize;
	uint2 SourceTextureSubregionSize = InSourceTextureSubregion.zw - InSourceTextureSubregion.xy;
	uint2 SampleCoordinates = floor(InSourceTextureSubregion.xy + SampleCoordinatesNormalized * SourceTextureSubregionSize);
		
	float4 Sample = InSourceTexture.Load(int3(SampleCoordinates, 0));

#if IS_HEIGHTMAP
	OutValue = UnpackHeight(Sample.xy) / LANDSCAPE_MAX_VALUE;
#else // IS_HEIGHTMAP
	OutValue = Sample[InSourceTextureChannel];
#endif // !IS_HEIGHTMAP
}

#endif // MERGE_TEXTURE


// ----------------------------------------------------------------------------------

#if RESAMPLE_MERGED_TEXTURE

void ResampleMergedTexture(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	// Pixel coordinates to UV (SvPos is already offset by 0.5 to be at pixel center) :
	float2 UV = SVPos.xy / (float2)InRenderAreaSize;
	float4 SampleUV = float4(UV, 0.0f, 1.0f);
	SampleUV = mul(SampleUV, InOutputUVToMergedTextureUV);

	float Sample = 0.0f;
	if (all(SampleUV.xy >= 0.0f) && all(SampleUV.xy <= 1.0f))
	{
		Sample = InMergedTexture.Sample(InMergedTextureSampler, SampleUV.xy);
	}

	OutColor.x = Sample;

#if IS_HEIGHTMAP
	Sample *= LANDSCAPE_MAX_VALUE;

#if COMPRESS_HEIGHT
	OutColor.xy = PackHeight(Sample);
#endif // !COMPRESS_HEIGHT

#endif // IS_HEIGHTMAP
}

#endif // RESAMPLE_MERGED_TEXTURE


// ----------------------------------------------------------------------------------

#if PACK_RGBA_CHANNELS

void PackRGBAChannels(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	OutColor = 0;
	OutColor.x = InSourceTextures_0.Load(int3(SVPos.xy, 0)).x;
	if (InNumChannels > 0)
	{
		OutColor.y = InSourceTextures_1.Load(int3(SVPos.xy, 0)).x;
	}
	if (InNumChannels > 1)
	{
		OutColor.z = InSourceTextures_2.Load(int3(SVPos.xy, 0)).x;
	}
	if (InNumChannels > 2)
	{
		OutColor.w = InSourceTextures_3.Load(int3(SVPos.xy, 0)).x;
	}
}

#endif // PACK_RGBA_CHANNELS