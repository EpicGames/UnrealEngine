// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FLandscapeGrassWeightInterpolantsVSToPS
{
	float4 Position : SV_POSITION;
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 PositionPreOffset : TEXCOORD9;
#if VF_USE_PRIMITIVE_SCENE_DATA
	float LocalPosZ : TEXCOORD10;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
};

#if VERTEXSHADER

float2 RenderOffset;

/** Vertex Shader */
void VSMain(
	FVertexFactoryInput Input,
	out FLandscapeGrassWeightInterpolantsVSToPS Output
	)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	
	// We can't deal correctly with horizontal WPO. We can choose to either:
	// (i) only take the offset in the direction of the heightfield which means that grass can mismatch on some slopes, or
	// (ii) take the full offset which means that grass can be missing or corrupt at edges of each tile.
	// We choose (i) here.
	float3 WPO = GetMaterialWorldPositionOffset(VertexParameters);
	WPO = DFMultiplyVector(WPO, VertexFactoryGetWorldToLocal(VFIntermediates)) * float3(0, 0, 1);
	WPO = DFMultiplyVector(WPO, VertexFactoryGetLocalToWorld(VFIntermediates));
	WorldPosition.xyz += WPO;
#if VF_USE_PRIMITIVE_SCENE_DATA
	FDFVector3 AbsoluteWorldPosition = DFFastSubtract(WorldPosition.xyz, ResolvedView.PreViewTranslation);
	Output.LocalPosZ = DFMultiplyAffineDemote(AbsoluteWorldPosition, VertexFactoryGetWorldToLocal(VFIntermediates)).z;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
	Output.PositionPreOffset = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);
	Output.Position = Output.PositionPreOffset + float4(RenderOffset, 0, 0);
}

#elif PIXELSHADER

uint HeightGrassWeightChannelMask; // first 4 bits for HeightGrassWeightIndex0 (i.e. indicates whether a channel is active for the low 8 height bits), the next 4 bits for HeightGrassWeightIndex1 (same for the high 8 height bits), the rest is unused
uint4 PerGrassWeightChannelMask; // == 128 bits : 4 bits per grass weight x MaxGrassTypes (32 currently). Add more if MaxGrassTypes is increased

// FourBitsMask is a mask ([0, 3] range) to store whether a channel (RGBA) is active 
float4 FourBitsMaskToChannelMask(uint FourBitsMask)
{
	return float4(
		(FourBitsMask >> 0) & 1u,
		(FourBitsMask >> 1) & 1u,
		(FourBitsMask >> 2) & 1u,
		(FourBitsMask >> 3) & 1u);
}

// GrassWeightIndex is betweenin the [0, MaxGrassTypes[ range. This allows to store 4 bits per grass weight in a single uint4 (PerGrassWeightChannelMask) : 8 per uint
// This function retrieves the 4 bits mask (range [0, 15], hence the & 0xf masking) for a given grass weight index
uint GrassWeightIndexToFourBitsMask(uint GrassWeightIndex)
{
	return (PerGrassWeightChannelMask[GrassWeightIndex / 8] >> ((GrassWeightIndex % 8) * 4)) & 0xf;
}

// Pixel Shader
void PSMain( 
	FLandscapeGrassWeightInterpolantsVSToPS Interpolants
	OPTIONAL_IsFrontFace,
	out HALF4_TYPE OutColor	: SV_Target0
	)
{
	OutColor = 0;

	ResolvedView = ResolveView();

	// Build the SvPosition without RenderOffset so that we correctly reproject WorldPosition etc.
	float4 SvPositionPreOffset = Interpolants.PositionPreOffset / Interpolants.PositionPreOffset.w;
	SvPositionPreOffset.xy = (SvPositionPreOffset.xy * float2(0.5f, -0.5f) + 0.5f) * ResolvedView.ViewSizeAndInvSize.xy;

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants.FactoryInterpolants, SvPositionPreOffset);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPositionPreOffset, bIsFrontFace);

	OutColor = 0.0;

	if (HeightGrassWeightChannelMask != 0)
	{
		// Calculate and pack landscape height.
#if VF_USE_PRIMITIVE_SCENE_DATA
		float Height = clamp(round(Interpolants.LocalPosZ / TERRAIN_ZSCALE) + 32768.0, 0, 65535);
#else // !VF_USE_PRIMITIVE_SCENE_DATA
		float3 LocalPos = WSMultiplyDemote(GetWorldPosition(MaterialParameters), GetWorldToInstance(MaterialParameters));
		float Height = clamp(round(LocalPos.z / TERRAIN_ZSCALE) + 32768.0, 0, 65535);
#endif // VF_USE_PRIMITIVE_SCENE_DATA
		float2 PackedHeight = float2(floor(Height / 256), fmod(Height, 256)) / 255;

		OutColor += FourBitsMaskToChannelMask((HeightGrassWeightChannelMask >> 0) & 0xf) * PackedHeight.x;
		OutColor += FourBitsMaskToChannelMask((HeightGrassWeightChannelMask >> 4) & 0xf) * PackedHeight.y;
	}

	if (any(PerGrassWeightChannelMask) != 0)
	{	
#define COMPUTEGRASSWEIGHT(N) \
{ \
	uint FourBitsMask = GrassWeightIndexToFourBitsMask(N); \
	BRANCH \
	if (FourBitsMask != 0) \
	{ \
		OutColor += FourBitsMaskToChannelMask(FourBitsMask) * GetGrassWeight##N(MaterialParameters).x; \
	} \
}

#if HAVE_GetGrassWeight0
		COMPUTEGRASSWEIGHT(0);
#endif // HAVE_GetGrassWeight0
#if HAVE_GetGrassWeight1
		COMPUTEGRASSWEIGHT(1);
#endif // HAVE_GetGrassWeight1
#if HAVE_GetGrassWeight2
		COMPUTEGRASSWEIGHT(2);
#endif // HAVE_GetGrassWeight2
#if HAVE_GetGrassWeight3
		COMPUTEGRASSWEIGHT(3);
#endif // HAVE_GetGrassWeight3
#if HAVE_GetGrassWeight4
		COMPUTEGRASSWEIGHT(4);
#endif // HAVE_GetGrassWeight4
#if HAVE_GetGrassWeight5
		COMPUTEGRASSWEIGHT(5);
#endif // HAVE_GetGrassWeight5
#if HAVE_GetGrassWeight6
		COMPUTEGRASSWEIGHT(6);
#endif // HAVE_GetGrassWeight6
#if HAVE_GetGrassWeight7
		COMPUTEGRASSWEIGHT(7);
#endif // HAVE_GetGrassWeight7
#if HAVE_GetGrassWeight8
		COMPUTEGRASSWEIGHT(8);
#endif // HAVE_GetGrassWeight8
#if HAVE_GetGrassWeight9
		COMPUTEGRASSWEIGHT(9);
#endif // HAVE_GetGrassWeight9

#if HAVE_GetGrassWeight10
		COMPUTEGRASSWEIGHT(10);
#endif // HAVE_GetGrassWeight10
#if HAVE_GetGrassWeight11
		COMPUTEGRASSWEIGHT(11);
#endif // HAVE_GetGrassWeight11
#if HAVE_GetGrassWeight12
		COMPUTEGRASSWEIGHT(12);
#endif // HAVE_GetGrassWeight12
#if HAVE_GetGrassWeight13
		COMPUTEGRASSWEIGHT(13);
#endif // HAVE_GetGrassWeight13
#if HAVE_GetGrassWeight14
		COMPUTEGRASSWEIGHT(14);
#endif // HAVE_GetGrassWeight14
#if HAVE_GetGrassWeight15
		COMPUTEGRASSWEIGHT(15);
#endif // HAVE_GetGrassWeight15
#if HAVE_GetGrassWeight16
		COMPUTEGRASSWEIGHT(16);
#endif // HAVE_GetGrassWeight16
#if HAVE_GetGrassWeight17
		COMPUTEGRASSWEIGHT(17);
#endif // HAVE_GetGrassWeight17
#if HAVE_GetGrassWeight18
		COMPUTEGRASSWEIGHT(18);
#endif // HAVE_GetGrassWeight18
#if HAVE_GetGrassWeight19
		COMPUTEGRASSWEIGHT(19);
#endif // HAVE_GetGrassWeight19

#if HAVE_GetGrassWeight20
		COMPUTEGRASSWEIGHT(20);
#endif // HAVE_GetGrassWeight20
#if HAVE_GetGrassWeight21
		COMPUTEGRASSWEIGHT(21);
#endif // HAVE_GetGrassWeight21
#if HAVE_GetGrassWeight22
		COMPUTEGRASSWEIGHT(22);
#endif // HAVE_GetGrassWeight22
#if HAVE_GetGrassWeight23
		COMPUTEGRASSWEIGHT(23);
#endif // HAVE_GetGrassWeight23
#if HAVE_GetGrassWeight24
		COMPUTEGRASSWEIGHT(24);
#endif // HAVE_GetGrassWeight24
#if HAVE_GetGrassWeight25
		COMPUTEGRASSWEIGHT(25);
#endif // HAVE_GetGrassWeight25
#if HAVE_GetGrassWeight26
		COMPUTEGRASSWEIGHT(26);
#endif // HAVE_GetGrassWeight26
#if HAVE_GetGrassWeight27
		COMPUTEGRASSWEIGHT(27);
#endif // HAVE_GetGrassWeight27
#if HAVE_GetGrassWeight28
		COMPUTEGRASSWEIGHT(28);
#endif // HAVE_GetGrassWeight28
#if HAVE_GetGrassWeight29
		COMPUTEGRASSWEIGHT(29);
#endif // HAVE_GetGrassWeight29

#if HAVE_GetGrassWeight30
		COMPUTEGRASSWEIGHT(30);
#endif // HAVE_GetGrassWeight30
#if HAVE_GetGrassWeight31
		COMPUTEGRASSWEIGHT(31);
#endif // HAVE_GetGrassWeight31
	}
}
#endif