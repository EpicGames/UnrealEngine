// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "Landscape/LandscapeCommon.ush"

Texture2D ReadTexture1;
SamplerState ReadTexture1Sampler;
float4 SourceOffsetAndSizeUV;
uint ChannelSwizzleMask;

void CopyTextureVS(in uint VertexId : SV_VertexID, out float4 OutPosition : SV_POSITION, out float2 OutTexCoord : TEXCOORD0)
{
	OutTexCoord = float2(((VertexId + 1) / 3) & 1, VertexId & 1);
	OutPosition.xy = float2(OutTexCoord.x, 1.f - OutTexCoord.y) * 2.f - 1.f;
	OutPosition.zw = float2(0, 1);
}

void CopyTexturePS(in float4 InPosition : SV_POSITION, in noperspective float2 InUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	// InUV is in the [0,1] range but in the viewport space, whose size is CopySize, so we need to adjust it to sample the corresponding range in the source texture : 
	float2 SampleUV = (InUV * SourceOffsetAndSizeUV.zw) + SourceOffsetAndSizeUV.xy;
	float4 SampleColor = ReadTexture1.SampleLevel(ReadTexture1Sampler, SampleUV, 0);
	
	// Swizzle channels as requested by ChannelSwizzleMask:
	[unroll]
	for (uint DestinationChannelIndex = 0; DestinationChannelIndex < 4; ++DestinationChannelIndex)
	{
		uint SourceChannelIndex = ((3u << (DestinationChannelIndex * 2u)) & ChannelSwizzleMask) >> (DestinationChannelIndex * 2u);
		OutColor[DestinationChannelIndex] = SampleColor[SourceChannelIndex];
	}
}