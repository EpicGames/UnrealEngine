// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Material shader used to output the dominant landscape physical materials.
 */

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FLandscapeGrassWeightInterpolantsVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
};

#if VERTEXSHADER

/** Simple default vertex shader. */
void VSMain(
	FVertexFactoryInput Input,
	out FLandscapeGrassWeightInterpolantsVSToPS Output
)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
	Output.Position = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);

	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
}

#elif PIXELSHADER

float2 UpdateDominantMaterial(in float Index, in float Contribution, in float2 DominantMaterial)
{
	if (Contribution > DominantMaterial.y)
	{
		DominantMaterial.x = Index + 1; // Store indices as 1-based so that 0 means empty/default.
		DominantMaterial.y = Contribution;
	}
	return DominantMaterial;
}

/** Pixel shader outputs index of physical material. */
void PSMain(
	FLandscapeGrassWeightInterpolantsVSToPS Interpolants
	OPTIONAL_IsFrontFace,
	out HALF4_TYPE OutColor : SV_Target0
)
{
	ResolvedView = ResolveView();

	float4 SvPosition = Interpolants.Position;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants.FactoryInterpolants, SvPosition);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);

	float2 DomininantMaterial = float2(0, 0); // float2(MaterialIndex, Contribution)

#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 0
	DomininantMaterial = UpdateDominantMaterial(0, GetPhysicalMaterial0(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 1
	DomininantMaterial = UpdateDominantMaterial(1, GetPhysicalMaterial1(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 2
	DomininantMaterial = UpdateDominantMaterial(2, GetPhysicalMaterial2(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 3
	DomininantMaterial = UpdateDominantMaterial(3, GetPhysicalMaterial3(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 4
	DomininantMaterial = UpdateDominantMaterial(4, GetPhysicalMaterial4(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 5
	DomininantMaterial = UpdateDominantMaterial(5, GetPhysicalMaterial5(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 6
	DomininantMaterial = UpdateDominantMaterial(6, GetPhysicalMaterial6(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 7
	DomininantMaterial = UpdateDominantMaterial(7, GetPhysicalMaterial7(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 8
	DomininantMaterial = UpdateDominantMaterial(8, GetPhysicalMaterial8(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 9
	DomininantMaterial = UpdateDominantMaterial(9, GetPhysicalMaterial9(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 10
	DomininantMaterial = UpdateDominantMaterial(10, GetPhysicalMaterial10(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 11
	DomininantMaterial = UpdateDominantMaterial(11, GetPhysicalMaterial11(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 12
	DomininantMaterial = UpdateDominantMaterial(12, GetPhysicalMaterial12(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 13
	DomininantMaterial = UpdateDominantMaterial(13, GetPhysicalMaterial13(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 14
	DomininantMaterial = UpdateDominantMaterial(14, GetPhysicalMaterial14(MaterialParameters), DomininantMaterial);
#endif
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 15
	DomininantMaterial = UpdateDominantMaterial(15, GetPhysicalMaterial15(MaterialParameters), DomininantMaterial);
#endif
	
#if NUM_MATERIAL_OUTPUTS_GETPHYSICALMATERIAL > 16 // Corresponds to : UMaterialExpressionLandscapePhysicalMaterialOutput::MaxPhysicalMaterials == 16. 
	// We pick an arbitrary limit to support here, but the only cost of extending this is more typing.
#error Too many physical materials
#endif

	// Write index assuming R8_UNORM
	OutColor = float4(DomininantMaterial.x / 255.f, 0, 0, 0);
}

#endif
