// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LandscapeVertexFactory.usf: Landscape vertex factory.
=============================================================================*/

// Compile Version 2

#include "VertexFactoryCommon.ush"

#define VERTEX_FACTORY_MODIFIES_INTERPOLATION 1

#define VF_SUPPORTS_RAYTRACING_VERTEX_MASK 1

// Set by FLightMapTexturePolicy
#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"
#include "LightmapData.ush"

/* Return index into buffers for this section instance. */
int GetComponentLinearIndex()
{
#if FIXED_GRID
	// Fixed grid doesn't use LandscapeContinuousLODParameters buffers
	return 0;
#else
	return (LandscapeParameters.ComponentBaseY - LandscapeContinuousLODParameters.Min.y) * LandscapeContinuousLODParameters.Size.x + (LandscapeParameters.ComponentBaseX - LandscapeContinuousLODParameters.Min.x);
#endif
}

float GetViewLODData(uint ComponentIndex)
{
#if FIXED_GRID
	// Fixed grid doesn't use LandscapeContinuousLODParameters buffers
	return 0;
#else
	const int AbsoluteSectionLODIndex = View.LandscapeIndirection[LandscapeContinuousLODParameters.LandscapeIndex] + ComponentIndex;
	return View.LandscapePerComponentData[AbsoluteSectionLODIndex];
#endif
}

/* Get current Lod for this section instance. */
float GetSectionLod(uint ComponentIndex)
{
#if FIXED_GRID
	// Fixed grid has fixed Lod
	return LandscapeFixedGrid.LodValues.x;
#else
#if RAYTRACING_DYNAMIC_GEOMETRY_CONVERTER
	int LODBias = LandscapeParameters.RayTracingLODBias;
#else
	int LODBias = View.FarShadowStaticMeshLODBias;
#endif
	return min(GetViewLODData(ComponentIndex) + LODBias, LandscapeParameters.LastLOD);
#endif
}

/**
 * x = Current mesh LOD
 * y = Unused
 * z = SectionSizeQuads in current mesh LOD
 * w = 1/SectionSizeQuads in current mesh LOD
 **/
float4 GetLodValues(uint ComponentIndex)
{
#if FIXED_GRID 
	// Fixed grid has fixed Lod
	return LandscapeFixedGrid.LodValues;
#else
	// Get Lod for the current section and infer the values
	uint Lod = (uint)floor(GetSectionLod(ComponentIndex));
	float LodSubSectionSizeQuads = (float)((LandscapeParameters.SubsectionSizeVerts >> Lod) - 1);
	return float4(Lod, 0, LodSubSectionSizeQuads, rcp(LodSubSectionSizeQuads));
#endif
}

uint GetRayTracingSectionVertCountForLOD(uint ComponentIndex)
{
	uint Lod = (uint)floor(GetSectionLod(ComponentIndex));
	return ((LandscapeParameters.RayTracingSectionSizeVerts >> Lod) + 1);
}


/**
 * Return Heightmap texture LOD difference from current LOD to highest LOD
 */
float GetLodBias(uint ComponentIndex)
{
#if FIXED_GRID
	// Fixed grid doesn't use LodBias
	return 0;
#else
	return LandscapeContinuousLODParameters.SectionLODBias[ComponentIndex];
#endif
}



float GetNeighborSectionLodFromOffset(int2 NeighborOffset, float CenterLOD)
{
#if FIXED_GRID
	// Fixed grid has fixed Lod
	return LandscapeFixedGrid.LodValues.x;
#else // !FIXED_GRID
	// Calculate neighbor position, and clamp to valid range
	int2 NeighborPos = NeighborOffset + int2(LandscapeParameters.ComponentBaseX, LandscapeParameters.ComponentBaseY);
	NeighborPos = max(LandscapeContinuousLODParameters.Min                                                     , NeighborPos);
	NeighborPos = min(LandscapeContinuousLODParameters.Min + LandscapeContinuousLODParameters.Size - int2(1, 1), NeighborPos);

	// lookup neighbor LOD
	uint NeighborComponentIndex = (NeighborPos.y - LandscapeContinuousLODParameters.Min.y) * LandscapeContinuousLODParameters.Size.x + (NeighborPos.x - LandscapeContinuousLODParameters.Min.x);
	float NeighborLOD = GetSectionLod(NeighborComponentIndex);

	// the Edge LOD is the max of Our LOD and the Neighbor LOD (both sides will agree on EdgeLOD this way)
	float EdgeLOD = max(CenterLOD, NeighborLOD);
	
	return EdgeLOD;
#endif // !FIXED_GRID
}

/** Calculate fractional render LOD for a vertex within a subsection. */
float CalcLOD(uint ComponentIndex, float2 xyLocalToSubsection, float2 Subsection)
{
#if FIXED_GRID
	// Fixed grid has fixed Lod
	return LandscapeFixedGrid.LodValues.x;
#else
	float2 xy = (xyLocalToSubsection + Subsection) / LandscapeParameters.NumSubsections;
	
	// calculate blend value (normalized distance from section border: 1 at the border, 0 at the center)
	float2 Delta = xy * 2 - 1;
	float2 AbsDelta = abs(Delta);
	float LB = max(AbsDelta.x, AbsDelta.y);

	// shift the blend towards 0 to reduce the lod blend range
	float k = LandscapeParameters.InvLODBlendRange;
	LB = saturate(1 - (1-LB) * k);

	// determine which neighbor is closest
	int2 NeighborOffset = (AbsDelta.x > AbsDelta.y) ? int2(sign(Delta.x), 0) : int2(0, sign(Delta.y));

	// calculate lod for this section, and the closest neighbor
	float CenterLod = GetSectionLod(ComponentIndex);
	float NeighborLod = GetNeighborSectionLodFromOffset(NeighborOffset, CenterLod);

	// blend from center lod to the neighbor lod, based on distance from the border
	float LODCalculated = (1-LB) * CenterLod + LB * NeighborLod;

	return LODCalculated;
#endif
}

struct FVertexFactoryInput
{
	// UByte4
	uint4 Position : ATTRIBUTE0;

#if LANDSCAPE_TILE
	#undef GetInstanceIdFromVF
	// UByte4
	uint4 TileData : ATTRIBUTE1;
#if INSTANCED_STEREO
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId : SV_InstanceID;
#else
	#define GetInstanceIdFromVF(VFInput)				(0)
#endif // INSTANCED_STEREO
#else
	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE1
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK() 
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

// RHI_RAYTRACING
#if RAYHITGROUPSHADER
FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input;
		
	uint RTSectionSizeVertsCurrentLOD = GetRayTracingSectionVertCountForLOD(GetComponentLinearIndex());
	uint RTSectionSizeQuadsCurrentLOD = RTSectionSizeVertsCurrentLOD - 1;
	uint NumRayTracingSections = LandscapeParameters.NumRayTracingSections;

	float RTSectionSizeScaler = (RTSectionSizeQuadsCurrentLOD - 1.0f / NumRayTracingSections) / RTSectionSizeQuadsCurrentLOD ;
	
	#if 1
		FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);
		uint VertexId = Tri.Indices[VertexIndex];
	#else
		uint QuadId = TriangleIndex / 2;
		uint QuadX = QuadId % RTSectionSizeQuadsCurrentLOD;
		uint QuadY = QuadId / RTSectionSizeQuadsCurrentLOD;
		uint i00 = (QuadX + 0) + (QuadY + 0) * RTSectionSizeVertsCurrentLOD;
		uint i10 = (QuadX + 1) + (QuadY + 0) * RTSectionSizeVertsCurrentLOD;
		uint i11 = (QuadX + 1) + (QuadY + 1) * RTSectionSizeVertsCurrentLOD;
		uint i01 = (QuadX + 0) + (QuadY + 1) * RTSectionSizeVertsCurrentLOD;
		uint Indices[6] = {i00, i11, i10, i00, i01, i10};
		uint VertexId = Indices[TriangleIndex % 2 * 3 + VertexIndex];
	#endif

	// Note: GetInstanceUserData() stores the GPU-Scene instance ID
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GetInstanceUserData(), 0U);

	Input.Position.x = (VertexId % RTSectionSizeVertsCurrentLOD + LandscapeMVF.RayTracingSectionXY.x * RTSectionSizeQuadsCurrentLOD) * RTSectionSizeScaler;
	Input.Position.y = (VertexId / RTSectionSizeVertsCurrentLOD + LandscapeMVF.RayTracingSectionXY.y * RTSectionSizeQuadsCurrentLOD) * RTSectionSizeScaler;	
	Input.Position.z = LandscapeMVF.SubXY.x;
	Input.Position.w = LandscapeMVF.SubXY.y;

	return Input;
}
#endif

#if COMPUTESHADER
FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;
	
	uint RTSectionSizeVertsCurrentLOD = GetRayTracingSectionVertCountForLOD(GetComponentLinearIndex());
	uint RTSectionSizeQuadsCurrentLOD = RTSectionSizeVertsCurrentLOD - 1;
	uint NumRayTracingSections = LandscapeParameters.NumRayTracingSections;

	float RTSectionSizeScaler = (RTSectionSizeQuadsCurrentLOD - 1.0f / NumRayTracingSections) / RTSectionSizeQuadsCurrentLOD ;

	uint VertexId = TriangleIndex * 3 + VertexIndex;

	Input.Position.x = (VertexId % RTSectionSizeVertsCurrentLOD + LandscapeMVF.RayTracingSectionXY.x * RTSectionSizeQuadsCurrentLOD) * RTSectionSizeScaler;
	Input.Position.y = (VertexId / RTSectionSizeVertsCurrentLOD + LandscapeMVF.RayTracingSectionXY.y * RTSectionSizeQuadsCurrentLOD) * RTSectionSizeScaler;	
	Input.Position.z = LandscapeMVF.SubXY.x;
	Input.Position.w = LandscapeMVF.SubXY.y;

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, PrimitiveData.InstanceSceneDataOffset + DrawInstanceId, DrawInstanceId);

	return Input;
}

uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif

struct FVertexFactoryInterpolantsVSToPS
{
	float2	LayerTexCoord			: TEXCOORD0; // xy == texcoord
	float4	WeightHeightMapTexCoord	: TEXCOORD1;
	float4	TransformedTexCoords	: TEXCOORD2;
#if NEEDS_LIGHTMAP_COORDINATE
#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
	float2	LightMapCoordinate[2]	: TEXCOORD4;
	float2	ShadowMapCoordinate		: TEXCOORD6;
#else
	float2	LightMapCoordinate		: TEXCOORD3;
	float2	ShadowMapCoordinate		: TEXCOORD4;
#endif
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	nointerpolation uint PrimitiveId : PRIMITIVE_ID;
#endif
};

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	return 0;
}
#endif

struct FLandscapeTexCoords
{
	float2	LayerTexCoord : TEXCOORD0; // xy == texcoord
	float2	WeightMapTexCoord;
	float2	HeightMapTexCoord;
#if NEEDS_LIGHTMAP_COORDINATE
	float2	LightMapCoordinate;
#endif
};

struct FVertexFactoryRayTracingInterpolants
{
	FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	
#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	// First row of the tangent to world matrix
	float3	TangentToWorld0	: VS_To_DS_TangentToWorld0;
	// Last row of the tangent to world matrix in xyz
	float4	TangentToWorld2	: VS_To_DS_TangentToWorld2;
	// LOD of the vertex, used for fading out tessellation
	float CalculatedLOD : VS_To_DS_CalculatedLOD;
#endif
};

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
{
	return Interpolants.InterpolantsVSToPS.WeightHeightMapTexCoord.zw;
}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
{
	return Input.InterpolantsVSToPS;
}

struct FVertexFactoryIntermediates
{
	float4 InputPosition;
	float3 LocalPosition;
	float3 WorldNormal;
	uint ComponentIndex;
	float4 LodValues;
	float LodBias;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

FInstanceSceneData GetInstanceData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.InstanceData;
}

float3 GetLocalPosition(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.LocalPosition+float3(Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.ww,0);
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;
	return TransformLocalToTranslatedWorld(GetLocalPosition(Intermediates), LocalToWorld);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return GetLocalPosition(Intermediates);
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.WorldNormal;
}

float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	// Note we still use LocalToWorld. Landscape's transform never changes.
	float3 LocalPosition = GetLocalPosition(Intermediates);
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;
	return TransformPreviousLocalPositionToTranslatedWorld(LocalPosition, LocalToWorld);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return GetLocalPosition(Intermediates);
}

/** Calculate the texture coordinates generated by Landscape */
FLandscapeTexCoords GetLandscapeTexCoords(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	FLandscapeTexCoords Result;

	Result.LayerTexCoord.xy = Intermediates.LocalPosition.xy + LandscapeParameters.SubsectionSizeVertsLayerUVPan.zw + Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.ww;

	Result.WeightMapTexCoord = Intermediates.LocalPosition.xy * LandscapeParameters.WeightmapUVScaleBias.xy + LandscapeParameters.WeightmapUVScaleBias.zw + Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.zz;
	Result.HeightMapTexCoord = Intermediates.LocalPosition.xy * LandscapeParameters.HeightmapUVScaleBias.xy + LandscapeParameters.HeightmapUVScaleBias.zw + 0.5*LandscapeParameters.HeightmapUVScaleBias.xy + Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.xy;

#if NEEDS_LIGHTMAP_COORDINATE
	Result.LightMapCoordinate = (Intermediates.LocalPosition.xy * LandscapeParameters.LandscapeLightmapScaleBias.xy + LandscapeParameters.LandscapeLightmapScaleBias.wz + Intermediates.InputPosition.zw * LandscapeParameters.LightmapSubsectionOffsetParams.xy);
#endif
	
	return Result;
}

float3x3 CalcTangentBasisFromWorldNormal(float3 Normal)
{
	float3 LocalTangentX = normalize(float3(Normal.z, 0.0f, -Normal.x));
	float3 LocalTangentY = cross(Normal, LocalTangentX);

	float3x3 LocalToTangent = float3x3(LocalTangentX,LocalTangentY,Normal);

	return LocalToTangent;
}

/** Lookup per-pixel tangent basis from heightmap texture */
float3x3 VertexFactoryGetPerPixelTangentBasis(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	float3x3 Result;
#if PIXELSHADER || RAYHITGROUPSHADER || COMPUTESHADER
	float4 SampleValue = Texture2DSample(LandscapeParameters.NormalmapTexture, LandscapeParameters.NormalmapTextureSampler, Interpolants.WeightHeightMapTexCoord.zw);
	float2 SampleNormal = float2(SampleValue.b, SampleValue.a) * float2(2.0,2.0) - float2(1.0,1.0);
	float3 WorldNormal = float3( SampleNormal, sqrt(max(1.0-dot(SampleNormal,SampleNormal),0.0)) );

	Result = CalcTangentBasisFromWorldNormal(WorldNormal);
#endif
	return Result;
}

/** Lookup per-pixel height from heightmap texture. */
#define VF_PER_PIXEL_HEIGHTMAP 1
/** Use high quality barycentric version of interpolation. */
#ifndef PER_PIXEL_HEIGHTMAP_HQ
#define PER_PIXEL_HEIGHTMAP_HQ 0
#endif

bool HasVertexFactoryPerPixelHeight()
{
	return LandscapeParameters.VirtualTexturePerPixelHeight != 0;
}

float GetVertexFactoryPerPixelHeight(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	float2 UV = Interpolants.WeightHeightMapTexCoord.zw;

#if PER_PIXEL_HEIGHTMAP_HQ
	if (LandscapeParameters.VirtualTexturePerPixelHeight > 1)
	{
		// It only makes sense to use barycentric sampling when sampling resolution is sufficiently above heightmap resolution.
		float2 TextureSize = LandscapeParameters.HeightmapTextureSize.xy;
		float2 dUVdx = ddx(UV * TextureSize);
		float2 dUVdy = ddy(UV * TextureSize);
		float T = max(dot(dUVdx, dUVdx), dot(dUVdy, dUVdy));
		if (T < 0.125f)
		{
			// Fake barycentric interpolation to try and match same result that we would get from vertex interpolation with a vertex located at each texel.
			float2 PixelCoord = UV * TextureSize - 0.5f;
			float2 FloorPixelCoord = floor(PixelCoord);
			float2 SubPixelPos = PixelCoord - FloorPixelCoord;

			float B1 = min(SubPixelPos.x, SubPixelPos.y);
			float B2 = abs(SubPixelPos.y - SubPixelPos.x);
			float B0 = 1.0f - B1 - B2;

			// Which is the third sample/triangle vertex depends on if we are upper or lower triangle in the quad.
			bool bLowerTriangle = (SubPixelPos.x > SubPixelPos.y);

			float2 InvTextureSize = LandscapeParameters.HeightmapTextureSize.zw;
			float2 BaseUV = (FloorPixelCoord + 0.5f) * InvTextureSize;
			float2 UV1 = InvTextureSize;
			float2 UV2 = bLowerTriangle ? float2(InvTextureSize.x, 0) : float2(0, InvTextureSize.y);

			float2 S0 = LandscapeParameters.HeightmapTexture.SampleLevel(LandscapeParameters.HeightmapTextureSampler, BaseUV, 0).xy;
			float2 S1 = LandscapeParameters.HeightmapTexture.SampleLevel(LandscapeParameters.HeightmapTextureSampler, BaseUV + UV1, 0).xy;
			float2 S2 = LandscapeParameters.HeightmapTexture.SampleLevel(LandscapeParameters.HeightmapTextureSampler, BaseUV + UV2, 0).xy;

			float H0 = DecodePackedHeight(S0);
			float H1 = DecodePackedHeight(S1);
			float H2 = DecodePackedHeight(S2);

			return B0 * H0 + B1 * H1 + B2 * H2;
		}
	}
#endif // PER_PIXEL_HEIGHTMAP_HQ

	// Use simple bilinear interpolation of heightmap.
	// Ideally we want to reuse this sample from VertexFactoryGetPerPixelTangentBasis().
	float4 SampleValue = LandscapeParameters.NormalmapTexture.Sample(LandscapeParameters.NormalmapTextureSampler, UV);
	return DecodePackedHeight(SampleValue.xy);
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NEEDS_LIGHTMAP_COORDINATE
	#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
		// Not supported in pixel shader
		float2 LightmapUVs = float2(0, 0);
	#else
		float2 LightmapUVs = Interpolants.LightMapCoordinate.xy;
	#endif
#else
	float2 LightmapUVs = float2(0, 0);
#endif

#if NUM_MATERIAL_TEXCOORDS     // XY layer
	Result.TexCoords[0] = Interpolants.LayerTexCoord.xy;
#if NUM_MATERIAL_TEXCOORDS > 1 // VS calcualted TexCoord 1, default is XZ layer
	Result.TexCoords[1] = Interpolants.TransformedTexCoords.xy;
#if NUM_MATERIAL_TEXCOORDS > 2 // VS calcualted TexCoord 2, default is YZ layer
	Result.TexCoords[2] = Interpolants.TransformedTexCoords.zw;
#if NUM_MATERIAL_TEXCOORDS > 3 // Weightmap
	Result.TexCoords[3] = Interpolants.WeightHeightMapTexCoord.xy;
#if NUM_MATERIAL_TEXCOORDS > 4 // Lightmap
	#if NEEDS_LIGHTMAP_COORDINATE
		Result.TexCoords[4] = LightmapUVs;
	#else
		Result.TexCoords[4] = float2(0,0);
	#endif
#if NUM_MATERIAL_TEXCOORDS > 5 // Height map
	Result.TexCoords[5] = Interpolants.WeightHeightMapTexCoord.zw;
#if NUM_MATERIAL_TEXCOORDS > 6
	for (uint CoordinateIndex = 6; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = float2(0,0);
	}
#endif // 6
#endif // 5
#endif // 4
#endif // 3
#endif // 2
#endif // 1
#endif // 0

	// Calculate LocalToTangent directly from normal map texture.
	float3x3 TangentToLocal = VertexFactoryGetPerPixelTangentBasis(Interpolants);
	Result.TangentToWorld = mul(TangentToLocal, (float3x3)LandscapeParameters.LocalToWorldNoScaling);
	Result.UnMirrored = 1;

	Result.VertexColor = 1;

#if LIGHTMAP_UV_ACCESS
	Result.LightmapUVs = LightmapUVs;
#endif

	Result.TwoSidedSign = 1;

#if VF_USE_PRIMITIVE_SCENE_DATA
	Result.PrimitiveId = Interpolants.PrimitiveId;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = float4(1,1,1,1);
	Result.TangentToWorld = mul(TangentToLocal, (float3x3)LandscapeParameters.LocalToWorldNoScaling);
	Result.PreSkinnedPosition = Intermediates.LocalPosition.xyz;
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

	// Assumes no instancing and landscape's transform never change
	Result.PrevFrameLocalToWorld = GetInstanceData(Intermediates).LocalToWorld;

	FLandscapeTexCoords LandscapeTexCoords = GetLandscapeTexCoords(Input, Intermediates);

#if NUM_MATERIAL_TEXCOORDS_VERTEX     // XY layer
	Result.TexCoords[0] = LandscapeTexCoords.LayerTexCoord.xy;
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1 // XZ layer
	Result.TexCoords[1] = float2(LandscapeTexCoords.LayerTexCoord.x, Intermediates.LocalPosition.z);
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2 // YZ layer
	Result.TexCoords[2] = float2(LandscapeTexCoords.LayerTexCoord.y, Intermediates.LocalPosition.z);
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3 // Weightmap
	Result.TexCoords[3] = LandscapeTexCoords.WeightMapTexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4 // Lightmap
	#if NEEDS_LIGHTMAP_COORDINATE
		Result.TexCoords[4] = LandscapeTexCoords.LightMapCoordinate.xy;
	#else
		Result.TexCoords[4] = float2(0,0);
	#endif
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5 // Height map
	Result.TexCoords[5] = LandscapeTexCoords.HeightMapTexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 6
	UNROLL
	for (uint CoordinateIndex = 6; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = float2(0,0);
	}
#endif // 6
#endif // 5
#endif // 4
#endif // 3
#endif // 2
#endif // 1
#endif // 0

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

#if NEEDS_LIGHTMAP_COORDINATE
void GetLightMapCoordinates(FVertexFactoryInterpolantsVSToPS Interpolants, out float2 LightmapUV0, out float2 LightmapUV1, out uint LightmapDataIndex)
{
#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
	LightmapUV0 = Interpolants.LightMapCoordinate[0];
	LightmapUV1 = Interpolants.LightMapCoordinate[1];
#else
	LightmapUV0 = Interpolants.LightMapCoordinate * float2( 1, 0.5 );
	LightmapUV1 = LightmapUV0 + float2( 0, 0.5 );
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	LightmapDataIndex = GetPrimitiveData(Interpolants.PrimitiveId).LightmapDataIndex;
#else
	LightmapDataIndex = 0;
#endif
}

void GetShadowMapCoordinate(FVertexFactoryInterpolantsVSToPS Interpolants, out float2 ShadowMapCoordinate, out uint LightmapDataIndex)
{
	ShadowMapCoordinate = Interpolants.ShadowMapCoordinate;
	
#if VF_USE_PRIMITIVE_SCENE_DATA
	LightmapDataIndex = GetPrimitiveData(Interpolants.PrimitiveId).LightmapDataIndex;
#else
	LightmapDataIndex = 0;
#endif

}
#endif

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.ComponentIndex = GetComponentLinearIndex();
	Intermediates.LodValues = GetLodValues(Intermediates.ComponentIndex);
	Intermediates.LodBias = GetLodBias(Intermediates.ComponentIndex);
	Intermediates.InputPosition = Input.Position;

#if LANDSCAPE_TILE   
	Intermediates.InputPosition = Intermediates.InputPosition + Input.TileData;
	Intermediates.InputPosition.xy = min(Intermediates.InputPosition.xy, float2(Intermediates.LodValues.z, Intermediates.LodValues.z));
#endif

	float2 xy = Intermediates.InputPosition.xy * Intermediates.LodValues.w;

	float LODCalculated = CalcLOD(Intermediates.ComponentIndex, xy, Intermediates.InputPosition.zw);
	float LodValue = floor(LODCalculated);
	float MorphAlpha = LODCalculated - LodValue;

	// InputPositionLODAdjusted : Position for actual LOD in base LOD units
	float2 ActualLODCoordsInt = floor(Intermediates.InputPosition.xy * pow(2, -(LodValue - Intermediates.LodValues.x)));
	float InvLODScaleFactor = pow(2, -LodValue);

	// Base to Actual LOD, Base to Next LOD
	float2 CoordTranslate = float2( LandscapeParameters.SubsectionSizeVertsLayerUVPan.x * InvLODScaleFactor - 1, max(LandscapeParameters.SubsectionSizeVertsLayerUVPan.x * 0.5f * InvLODScaleFactor, 2) - 1 ) * LandscapeParameters.SubsectionSizeVertsLayerUVPan.y;
	float2 InputPositionLODAdjusted = ActualLODCoordsInt / CoordTranslate.x;

	// InputPositionNextLOD : Position for next LOD in base LOD units
	float2 NextLODCoordsInt = floor(ActualLODCoordsInt * 0.5);
	float2 InputPositionNextLOD = NextLODCoordsInt / CoordTranslate.y;

	// Get the height and normal XY for current and next LOD out of the textures
	float2 SampleCoords = InputPositionLODAdjusted * LandscapeParameters.HeightmapUVScaleBias.xy + LandscapeParameters.HeightmapUVScaleBias.zw + 0.5*LandscapeParameters.HeightmapUVScaleBias.xy + Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.xy;
	float4 SampleValue = Texture2DSampleLevel(LandscapeParameters.HeightmapTexture, LandscapeParameters.HeightmapTextureSampler, SampleCoords, LodValue-Intermediates.LodBias);
	float Height = DecodePackedHeight(SampleValue.xy);

	float2 SampleCoordsNextLOD = InputPositionNextLOD * LandscapeParameters.HeightmapUVScaleBias.xy + LandscapeParameters.HeightmapUVScaleBias.zw + 0.5*LandscapeParameters.HeightmapUVScaleBias.xy + Intermediates.InputPosition.zw * LandscapeParameters.SubsectionOffsetParams.xy;
	float4 SampleValueNextLOD = Texture2DSampleLevel(LandscapeParameters.HeightmapTexture, LandscapeParameters.HeightmapTextureSampler, SampleCoordsNextLOD, LodValue+1-Intermediates.LodBias);
	float HeightNextLOD = DecodePackedHeight(SampleValueNextLOD.xy);

	Intermediates.LocalPosition = lerp( float3(InputPositionLODAdjusted, Height), float3(InputPositionNextLOD, HeightNextLOD), MorphAlpha );

	float2 Normal = float2(SampleValue.b, SampleValue.a);
	float2 NormalNextLOD = float2(SampleValueNextLOD.b, SampleValueNextLOD.a);
	float2 InterpNormal = lerp( Normal, NormalNextLOD, MorphAlpha ) * float2(2.0,2.0) - float2(1.0,1.0);
	Intermediates.WorldNormal = float3( InterpNormal, sqrt(max(1.0-dot(InterpNormal,InterpNormal),0.0)) );
	
	return Intermediates;
}


/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3x3 Result = CalcTangentBasisFromWorldNormal(Intermediates.WorldNormal);
	return Result;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
	FLandscapeTexCoords LandscapeTexCoords = GetLandscapeTexCoords(Input, Intermediates);

	Interpolants.LayerTexCoord				= LandscapeTexCoords.LayerTexCoord;
	Interpolants.WeightHeightMapTexCoord.xy	= LandscapeTexCoords.WeightMapTexCoord;
	Interpolants.WeightHeightMapTexCoord.zw	= LandscapeTexCoords.HeightMapTexCoord;

	// Landscape material doesn't support custom vertex interpolators for now but GetMaterialCustomizedUVs is declared using NUM_TEX_COORD_INTERPOLATORS
#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
#endif // NUM_TEX_COORD_INTERPOLATORS

	// Landscape material only supports up to NUM_MATERIAL_TEXCOORDS 5 (0-3 are reserved for landscape-specific stuff: see UMaterialExpressionLandscapeLayerCoords, 4 is for lightmaps and 5 for heightmap coordinates, 
	//  anything above is not supported : see GetMaterialPixelParameters) : 
#if NUM_MATERIAL_TEXCOORDS
	Interpolants.LayerTexCoord.xy = CustomizedUVs[0];
	Interpolants.TransformedTexCoords = 0;

#if NUM_MATERIAL_TEXCOORDS > 1
	Interpolants.TransformedTexCoords.xy = CustomizedUVs[1];
#if NUM_MATERIAL_TEXCOORDS > 2
	Interpolants.TransformedTexCoords.zw = CustomizedUVs[2];
#endif // NUM_MATERIAL_TEXCOORDS > 2
#endif // NUM_MATERIAL_TEXCOORDS > 1
#endif // NUM_MATERIAL_TEXCOORDS

#if NEEDS_LIGHTMAP_COORDINATE
	#if VF_USE_PRIMITIVE_SCENE_DATA
		const uint LightmapDataIndex = GetPrimitiveData(Intermediates).LightmapDataIndex;
	#else
		const uint LightmapDataIndex = 0;
	#endif
	
	#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
		Interpolants.LightMapCoordinate[0] = LandscapeTexCoords.LightMapCoordinate * VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).xy + VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).zw;
		Interpolants.LightMapCoordinate[0].y *= 0.5;
		Interpolants.LightMapCoordinate[1] = Interpolants.LightMapCoordinate[0].xy;
		Interpolants.LightMapCoordinate[1].y += 0.5;
	#else
		Interpolants.LightMapCoordinate = LandscapeTexCoords.LightMapCoordinate * VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).xy + VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).zw;
	#endif
	
	#if STATICLIGHTING_TEXTUREMASK
		Interpolants.ShadowMapCoordinate = LandscapeTexCoords.LightMapCoordinate * VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).xy + VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(Input, LightmapDataIndex).zw;
	#else
		Interpolants.ShadowMapCoordinate = 0;
	#endif
#endif
	
#if VF_USE_PRIMITIVE_SCENE_DATA
	Interpolants.PrimitiveId = Intermediates.SceneData.PrimitiveId;
#endif

	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryRayTracingInterpolants Interpolants;
	
	Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	// Calculate LocalToTangent directly from normal map texture.
	float3x3 TangentToLocal = CalcTangentBasisFromWorldNormal(Intermediates.WorldNormal);
	float3x3 TangentToWorld = mul(TangentToLocal, (float3x3)LandscapeParameters.LocalToWorldNoScaling);
	
	Interpolants.TangentToWorld0 = TangentToWorld[0];
	Interpolants.TangentToWorld2 = float4(TangentToWorld[2], 1);

	float2 xy = Intermediates.InputPosition.xy * Intermediates.LodValues.w;
	Interpolants.CalculatedLOD = CalcLOD(Intermediates.ComponentIndex, xy, Intermediates.InputPosition.zw);
#endif
	
	return Interpolants;
}

FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
{
	FVertexFactoryRayTracingInterpolants O;
	
#if NEEDS_LIGHTMAP_COORDINATE && FEATURE_LEVEL > FEATURE_LEVEL_ES3_1
	INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapCoordinate);
#endif

	INTERPOLATE_MEMBER(InterpolantsVSToPS.LayerTexCoord);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.WeightHeightMapTexCoord);
	INTERPOLATE_MEMBER(InterpolantsVSToPS.TransformedTexCoords);

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	INTERPOLATE_MEMBER(TangentToWorld0);
	INTERPOLATE_MEMBER(TangentToWorld2);
	INTERPOLATE_MEMBER(CalculatedLOD);
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	O.InterpolantsVSToPS.PrimitiveId = a.InterpolantsVSToPS.PrimitiveId;
#endif

	return O;
}

#if NUM_VF_PACKED_INTERPOLANTS > 0
void VertexFactoryPackInterpolants(inout FVertexFactoryInterpolantsVSToPS Interpolants, float4 PackedInterpolants[NUM_VF_PACKED_INTERPOLANTS])
{
	Interpolants.TransformedTexCoord0.zw = PackedInterpolants[0].xy;
	Interpolants.TransformedTexCoord1.zw = PackedInterpolants[0].zw;
}

void VertexFactoryUnpackInterpolants(FVertexFactoryInterpolantsVSToPS Interpolants, out float4 PackedInterpolants[NUM_VF_PACKED_INTERPOLANTS])
{
	PackedInterpolants[0].xy = Interpolants.TransformedTexCoord0.zw;
	PackedInterpolants[0].zw = Interpolants.TransformedTexCoord1.zw;
#if NUM_VF_PACKED_INTERPOLANTS > 1
	UNROLL
	for (int i = 1; i < NUM_VF_PACKED_INTERPOLANTS; ++i)
	{
		PackedInterpolants[i] = 0;
	}
#endif
}
#endif // NUM_VF_PACKED_INTERPOLANTS > 0

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	return Interpolants.PrimitiveId;
#else // VF_USE_PRIMITIVE_SCENE_DATA
	return 0;
#endif // VF_USE_PRIMITIVE_SCENE_DATA
}

// Tell the VSM depth shader that we are supplying a constant bias
#define VF_USE_VSM_CONSTANT_BIAS 1

float VertexFactoryGetNonNaniteVirtualShadowMapConstantDepthBias()
{
	return LandscapeParameters.NonNaniteVirtualShadowMapConstantDepthBias;
}

#include "VertexFactoryDefaultInterface.ush"
