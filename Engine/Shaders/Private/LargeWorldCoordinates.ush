// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DoubleFloat.ush"

struct FLWCScalar
{
	float Tile;
	float Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	int Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

struct FLWCVector2
{
	float2 Tile;
	float2 Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	int Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

struct FLWCVector3
{
	float3 Tile;
	float3 Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	int Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

struct FLWCVector4
{
	float4 Tile;
	float4 Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	int Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

struct FLWCScalarDeriv
{
	FLWCScalar Value;
	float Ddx;
	float Ddy;
};

struct FLWCVector2Deriv
{
	FLWCVector2 Value;
	float2 Ddx;
	float2 Ddy;
};

struct FLWCVector3Deriv
{
	FLWCVector3 Value;
	float3 Ddx;
	float3 Ddy;
};

struct FLWCVector4Deriv
{
	FLWCVector4 Value;
	float4 Ddx;
	float4 Ddy;
};

// Transforms *to* absolute world space
struct FLWCMatrix
{
	float4x4 M;
	float3 Tile; // Added to result, *after* multiplying 'M'
#if !COMPILER_SUPPORTS_HLSL2021
	int2 Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

// Transforms *from* absolute world space
struct FLWCInverseMatrix
{
	float4x4 M;
	float3 Tile; // Added to input position *before* multiplying 'M'
#if !COMPILER_SUPPORTS_HLSL2021
	int3 Dummy; // avoid problem with HLSL overloading/implicit cast (should be DCE'd)
#endif
};

#define LWCSetTile(V, InTile) (V).Tile = (InTile)
#define LWCGetTile(V) ((V).Tile)

float LWCGetTileOffset(FLWCScalar V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }
float2 LWCGetTileOffset(FLWCVector2 V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }
float3 LWCGetTileOffset(FLWCVector3 V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }
float4 LWCGetTileOffset(FLWCVector4 V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }
float3 LWCGetTileOffset(FLWCMatrix V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }
float3 LWCGetTileOffset(FLWCInverseMatrix V) { return LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE; }

FLWCScalar MakeLWCScalar(float Tile, float Offset)
{
	FLWCScalar Result;
	LWCSetTile(Result, Tile);
	Result.Offset = Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCVector2 MakeLWCVector2(float2 Tile, float2 Offset)
{
	FLWCVector2 Result;
	LWCSetTile(Result, Tile);
	Result.Offset = Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCVector3 MakeLWCVector3(float3 Tile, float3 Offset)
{
	FLWCVector3 Result;
	LWCSetTile(Result, Tile);
	Result.Offset = Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCVector4 MakeLWCVector4(float4 Tile, float4 Offset)
{
	FLWCVector4 Result;
	LWCSetTile(Result, Tile);
	Result.Offset = Offset;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCVector4 MakeLWCVector4(float3 Tile, float4 Offset)
{
	return MakeLWCVector4(float4(Tile, 0), Offset);
}

FLWCVector4 MakeLWCVector4(FLWCVector3 XYZ, float W)
{
	return MakeLWCVector4(LWCGetTile(XYZ), float4(XYZ.Offset, W));
}

FLWCScalar MakeLWCVector(FLWCScalar X) { return X; }

FLWCVector2 MakeLWCVector(FLWCScalar X, FLWCScalar Y) { return MakeLWCVector2(float2(LWCGetTile(X), LWCGetTile(Y)), float2(X.Offset, Y.Offset)); }

FLWCVector3 MakeLWCVector(FLWCScalar X, FLWCScalar Y, FLWCScalar Z) { return MakeLWCVector3(float3(LWCGetTile(X), LWCGetTile(Y), LWCGetTile(Z)), float3(X.Offset, Y.Offset, Z.Offset)); }
FLWCVector3 MakeLWCVector(FLWCScalar X, FLWCVector2 YZ) { return MakeLWCVector3(float3(LWCGetTile(X), LWCGetTile(YZ)), float3(X.Offset, YZ.Offset)); }
FLWCVector3 MakeLWCVector(FLWCVector2 XY, FLWCScalar Z) { return MakeLWCVector3(float3(LWCGetTile(XY), LWCGetTile(Z)), float3(XY.Offset, Z.Offset)); }

FLWCVector4 MakeLWCVector(FLWCScalar X, FLWCScalar Y, FLWCScalar Z, FLWCScalar W) { return MakeLWCVector4(float4(LWCGetTile(X), LWCGetTile(Y), LWCGetTile(Z), LWCGetTile(W)), float4(X.Offset, Y.Offset, Z.Offset, W.Offset)); }
FLWCVector4 MakeLWCVector(FLWCScalar X, FLWCScalar Y, FLWCVector2 ZW) { return MakeLWCVector4(float4(LWCGetTile(X), LWCGetTile(Y), LWCGetTile(ZW)), float4(X.Offset, Y.Offset, ZW.Offset)); }
FLWCVector4 MakeLWCVector(FLWCScalar X, FLWCVector2 YZ, FLWCScalar W) { return MakeLWCVector4(float4(LWCGetTile(X), LWCGetTile(YZ), LWCGetTile(W)), float4(X.Offset, YZ.Offset, W.Offset)); }
FLWCVector4 MakeLWCVector(FLWCVector2 XY, FLWCScalar Z, FLWCScalar W) { return MakeLWCVector4(float4(LWCGetTile(XY), LWCGetTile(Z), LWCGetTile(W)), float4(XY.Offset, Z.Offset, W.Offset)); }
FLWCVector4 MakeLWCVector(FLWCVector2 XY, FLWCVector2 ZW) { return MakeLWCVector4(float4(LWCGetTile(XY), LWCGetTile(ZW)), float4(XY.Offset, ZW.Offset)); }
FLWCVector4 MakeLWCVector(FLWCScalar X, FLWCVector3 YZW) { return MakeLWCVector4(float4(LWCGetTile(X), LWCGetTile(YZW)), float4(X.Offset, YZW.Offset)); }
FLWCVector4 MakeLWCVector(FLWCVector3 XYZ, FLWCScalar W) { return MakeLWCVector4(float4(LWCGetTile(XYZ), LWCGetTile(W)), float4(XYZ.Offset, W.Offset)); }

FLWCMatrix MakeLWCMatrix(float3 Tile, float4x4 InMatrix)
{
	FLWCMatrix Result;
	LWCSetTile(Result, Tile);
	Result.M = InMatrix;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCMatrix MakeLWCMatrix4x3(float3 Tile, float4x4 InMatrix)
{
	FLWCMatrix Result;
	LWCSetTile(Result, Tile);
	Result.M = Make4x3Matrix(InMatrix);
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCInverseMatrix MakeLWCInverseMatrix(float3 Tile, float4x4 InMatrix)
{
	FLWCInverseMatrix Result;
	LWCSetTile(Result, -Tile);
	Result.M = InMatrix;
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

FLWCInverseMatrix MakeLWCInverseMatrix4x3(float3 Tile, float4x4 InMatrix)
{
	FLWCInverseMatrix Result;
	LWCSetTile(Result, -Tile);
	Result.M = Make4x3Matrix(InMatrix);
#if !COMPILER_SUPPORTS_HLSL2021
	Result.Dummy = 0;
#endif
	return Result;
}

// 'C' should typically be a constant value for optimized codegen; 0, 1, 2, 3 to select X, Y, Z, or W component
// Undefined if out-of-bounds
FLWCScalar LWCGetComponent(FLWCScalar V, int C) { return V; }
FLWCScalar LWCGetComponent(FLWCVector2 V, int C) { return MakeLWCScalar(LWCGetTile(V)[C], V.Offset[C]); }
FLWCScalar LWCGetComponent(FLWCVector3 V, int C) { return MakeLWCScalar(LWCGetTile(V)[C], V.Offset[C]); }
FLWCScalar LWCGetComponent(FLWCVector4 V, int C) { return MakeLWCScalar(LWCGetTile(V)[C], V.Offset[C]); }

#define LWCGetX(V) LWCGetComponent(V, 0)
#define LWCGetY(V) LWCGetComponent(V, 1)
#define LWCGetZ(V) LWCGetComponent(V, 2)
#define LWCGetW(V) LWCGetComponent(V, 3)

FLWCScalar LWCSwizzle(FLWCScalar V, int C0) { return V; }
FLWCScalar LWCSwizzle(FLWCVector2 V, int C0) { return LWCGetComponent(V, C0); }
FLWCScalar LWCSwizzle(FLWCVector3 V, int C0) { return LWCGetComponent(V, C0); }
FLWCScalar LWCSwizzle(FLWCVector4 V, int C0) { return LWCGetComponent(V, C0); }

FLWCVector2 LWCSwizzle(FLWCScalar V, int C0, int C1) { return MakeLWCVector(V, V); }
FLWCVector2 LWCSwizzle(FLWCVector2 V, int C0, int C1) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1)); }
FLWCVector2 LWCSwizzle(FLWCVector3 V, int C0, int C1) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1)); }
FLWCVector2 LWCSwizzle(FLWCVector4 V, int C0, int C1) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1)); }

FLWCVector3 LWCSwizzle(FLWCScalar V, int C0, int C1, int C2) { return MakeLWCVector(V, V, V); }
FLWCVector3 LWCSwizzle(FLWCVector2 V, int C0, int C1, int C2) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2)); }
FLWCVector3 LWCSwizzle(FLWCVector3 V, int C0, int C1, int C2) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2)); }
FLWCVector3 LWCSwizzle(FLWCVector4 V, int C0, int C1, int C2) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2)); }

FLWCVector4 LWCSwizzle(FLWCScalar V, int C0, int C1, int C2, int C3) { return MakeLWCVector(V, V, V, V); }
FLWCVector4 LWCSwizzle(FLWCVector2 V, int C0, int C1, int C2, int C3) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2), LWCGetComponent(V, C3)); }
FLWCVector4 LWCSwizzle(FLWCVector3 V, int C0, int C1, int C2, int C3) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2), LWCGetComponent(V, C3)); }
FLWCVector4 LWCSwizzle(FLWCVector4 V, int C0, int C1, int C2, int C3) { return MakeLWCVector(LWCGetComponent(V, C0), LWCGetComponent(V, C1), LWCGetComponent(V, C2), LWCGetComponent(V, C3)); }

float LWCToFloat(FLWCScalar Value)   { return LWCGetTileOffset(Value) + Value.Offset; }
float2 LWCToFloat(FLWCVector2 Value) { return LWCGetTileOffset(Value) + Value.Offset; }
float3 LWCToFloat(FLWCVector3 Value) { return LWCGetTileOffset(Value) + Value.Offset; }
float4 LWCToFloat(FLWCVector4 Value) { return LWCGetTileOffset(Value) + Value.Offset; }

float4x4 LWCToFloat(FLWCMatrix Value)
{
	float4x4 Result = Value.M;
	Result[3].xyz = LWCGetTileOffset(Value) + Result[3].xyz;
	return Result;
}

float4x4 LWCToFloat(FLWCInverseMatrix Value)
{
	float4x4 TileOffset = MakeTranslationMatrix(LWCGetTileOffset(Value));
	return mul(TileOffset, Value.M);
}

float3x3 LWCToFloat3x3(FLWCMatrix Value)
{
	return (float3x3)Value.M;
}

float3x3 LWCToFloat3x3(FLWCInverseMatrix Value)
{
	return (float3x3)Value.M;
}

// Allow LWCToFloat to be called on float values (as nop)
float LWCToFloat(float Value) { return Value; }
float2 LWCToFloat(float2 Value) { return Value; }
float3 LWCToFloat(float3 Value) { return Value; }
float4 LWCToFloat(float4 Value) { return Value; }
float4x4 LWCToFloat(float4x4 Value) { return Value; }

// 'LWCPromote' will convert a float value to LWC, or leave an LWC value as-is
FLWCScalar LWCPromote(FLWCScalar Value) { return Value; }
FLWCVector2 LWCPromote(FLWCVector2 Value) { return Value; }
FLWCVector3 LWCPromote(FLWCVector3 Value) { return Value; }
FLWCVector4 LWCPromote(FLWCVector4 Value) { return Value; }
FLWCMatrix LWCPromote(FLWCMatrix Value) { return Value; }
FLWCInverseMatrix LWCPromote(FLWCInverseMatrix Value) { return Value; }

FLWCScalar LWCPromote(float Value) { return MakeLWCScalar(0, Value); }
FLWCVector2 LWCPromote(float2 Value) { return MakeLWCVector2((float2)0, Value); }
FLWCVector3 LWCPromote(float3 Value) { return MakeLWCVector3((float3)0, Value); }
FLWCVector4 LWCPromote(float4 Value) { return MakeLWCVector4((float4)0, Value); }
FLWCMatrix LWCPromote(float4x4 Value) { return MakeLWCMatrix((float3)0, Value); }
FLWCInverseMatrix LWCPromoteInverse(float4x4 Value) { return MakeLWCInverseMatrix((float3)0, Value); }

FLWCVector3 LWCMultiply(float3 Position, FLWCMatrix InMatrix)
{
	// Explicit operations rather than mul() avoids z-fighting between depth/base passes
	float3 Offset = (Position.xxx * InMatrix.M[0].xyz + Position.yyy * InMatrix.M[1].xyz + Position.zzz * InMatrix.M[2].xyz) + InMatrix.M[3].xyz;
	return MakeLWCVector3(LWCGetTile(InMatrix), Offset);
}

FLWCVector4 LWCMultiply(float4 Position, FLWCMatrix InMatrix)
{
	float4 Offset = mul(Position, InMatrix.M);
	return MakeLWCVector4(LWCGetTile(InMatrix), Offset);
}

float3 LWCMultiply(FLWCVector3 Position, FLWCInverseMatrix InMatrix)
{
	float3 LocalPosition = LWCToFloat(MakeLWCVector3(LWCGetTile(Position) + LWCGetTile(InMatrix), Position.Offset));
	return (LocalPosition.xxx * InMatrix.M[0].xyz + LocalPosition.yyy * InMatrix.M[1].xyz + LocalPosition.zzz * InMatrix.M[2].xyz) + InMatrix.M[3].xyz;
}

float4 LWCMultiply(FLWCVector4 Position, FLWCInverseMatrix InMatrix)
{
	float4 LocalPosition = LWCToFloat(MakeLWCVector4(LWCGetTile(Position) + float4(LWCGetTile(InMatrix), 0), Position.Offset));
	return mul(LocalPosition, InMatrix.M);
}

float3 LWCMultiplyVector(float3 Vector, FLWCMatrix InMatrix)
{
	return mul(Vector, (float3x3)InMatrix.M);
}

float3 LWCMultiplyVector(float3 Vector, FLWCInverseMatrix InMatrix)
{
	return mul(Vector, (float3x3)InMatrix.M);
}

FLWCMatrix LWCMultiply(float4x4 Lhs, FLWCMatrix Rhs)
{
	float4x4 ResultMatrix = mul(Lhs, Rhs.M);
	return MakeLWCMatrix(LWCGetTile(Rhs), ResultMatrix);
}

FLWCInverseMatrix LWCMultiply(FLWCInverseMatrix Lhs, float4x4 Rhs)
{
	float4x4 ResultMatrix = mul(Lhs.M, Rhs);
	return MakeLWCInverseMatrix(-LWCGetTile(Lhs), ResultMatrix);
}

float4x4 LWCMultiply(FLWCMatrix Lhs, FLWCInverseMatrix Rhs)
{
	// Lhs.M * Lhs.Tile * Rhs.Tile * Rhs.M
	float4x4 Result = Lhs.M;
	Result = mul(Result, MakeTranslationMatrix((LWCGetTile(Lhs) + LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE));
	Result = mul(Result, Rhs.M);
	return Result;
}

float4x4 LWCMultiplyTranslation(FLWCMatrix Lhs, FLWCVector3 Rhs)
{
	float4x4 Result = Lhs.M;
	Result[3].xyz += (LWCGetTile(Lhs) + LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE;
	Result[3].xyz += Rhs.Offset;
	return Result;
}

FLWCMatrix LWCMultiplyTranslation(float4x4 Lhs, FLWCVector3 Rhs)
{
	FLWCMatrix Result = MakeLWCMatrix(LWCGetTile(Rhs), Lhs);
	Result.M[3].xyz += Rhs.Offset;
	return Result;
}

float4x4 LWCMultiplyTranslation(FLWCVector3 Lhs, FLWCInverseMatrix Rhs)
{
	float3 Offset = (LWCGetTile(Lhs) + LWCGetTile(Rhs)) * UE_LWC_RENDER_TILE_SIZE + Lhs.Offset;
	return mul(MakeTranslationMatrix(Offset), Rhs.M);
}

FLWCInverseMatrix LWCMultiplyTranslation(FLWCVector3 Lhs, float4x4 Rhs)
{
	FLWCInverseMatrix Result = MakeLWCInverseMatrix(-LWCGetTile(Lhs), Rhs);
	Result.M = mul(MakeTranslationMatrix(Lhs.Offset), Result.M);
	return Result;
}

FLWCVector3 LWCGetOrigin(FLWCMatrix InMatrix)
{
	return MakeLWCVector3(LWCGetTile(InMatrix), InMatrix.M[3].xyz);
}

void LWCSetOrigin(inout FLWCMatrix InOutMatrix, FLWCVector3 Origin)
{
	LWCSetTile(InOutMatrix, LWCGetTile(Origin));
	InOutMatrix.M[3].xyz = Origin.Offset;
}

#define FLWCType FLWCScalar
#define FFloatType float
#define FBoolType bool
#define LWCConstructor MakeLWCScalar
#include "LWCOperations.ush"
#undef FLWCType
#undef FFloatType
#undef FBoolType
#undef LWCConstructor

#define FLWCType FLWCVector2
#define FFloatType float2
#define FBoolType bool2
#define LWCConstructor MakeLWCVector2
#include "LWCOperations.ush"
#undef FLWCType
#undef FFloatType
#undef FBoolType
#undef LWCConstructor

#define FLWCType FLWCVector3
#define FFloatType float3
#define FBoolType bool3
#define LWCConstructor MakeLWCVector3
#include "LWCOperations.ush"
#undef FLWCType
#undef FFloatType
#undef FBoolType
#undef LWCConstructor

#define FLWCType FLWCVector4
#define FFloatType float4
#define FBoolType bool4
#define LWCConstructor MakeLWCVector4
#include "LWCOperations.ush"
#undef FLWCType
#undef FFloatType
#undef FBoolType
#undef LWCConstructor

// Creates a coordinate with the same value, but relative to the specified tile, while keeping precision loss as low as possible
FLWCScalar LWCMakeRelativeToTile(FLWCScalar V, float NewTile) { return MakeLWCScalar(NewTile, LWCToFloat(LWCSubtract(V, MakeLWCScalar(NewTile, (float)0.0f)))); }
FLWCVector2 LWCMakeRelativeToTile(FLWCVector2 V, float2 NewTile) { return MakeLWCVector2(NewTile, LWCToFloat(LWCSubtract(V, MakeLWCVector2(NewTile, (float2)0.0f)))); }
FLWCVector3 LWCMakeRelativeToTile(FLWCVector3 V, float3 NewTile) { return MakeLWCVector3(NewTile, LWCToFloat(LWCSubtract(V, MakeLWCVector3(NewTile, (float3)0.0f)))); }
FLWCVector4 LWCMakeRelativeToTile(FLWCVector4 V, float4 NewTile) { return MakeLWCVector4(NewTile, LWCToFloat(LWCSubtract(V, MakeLWCVector4(NewTile, (float4)0.0f)))); }
FLWCMatrix LWCMakeRelativeToTile(FLWCMatrix M, float3 NewTile)
{
	LWCSetOrigin(M, LWCMakeRelativeToTile(LWCGetOrigin(M), NewTile));
	return M;
}

FLWCScalar LWCVectorSum(FLWCScalar V) { return V; }
FLWCScalar LWCVectorSum(FLWCVector2 V) { return LWCAdd(LWCGetX(V), LWCGetY(V)); }
FLWCScalar LWCVectorSum(FLWCVector3 V) { return LWCAdd(LWCAdd(LWCGetX(V), LWCGetY(V)), LWCGetZ(V)); }
FLWCScalar LWCVectorSum(FLWCVector4 V) { return LWCAdd(LWCAdd(LWCAdd(LWCGetX(V), LWCGetY(V)), LWCGetZ(V)), LWCGetW(V)); }

FLWCScalar LWCDot(FLWCScalar Lhs, FLWCScalar Rhs) { return LWCMultiply(Lhs, Rhs); }
FLWCScalar LWCDot(FLWCScalar Lhs, float Rhs) { return LWCMultiply(Lhs, Rhs); }
FLWCScalar LWCDot(FLWCVector2 Lhs, FLWCVector2 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }
FLWCScalar LWCDot(FLWCVector2 Lhs, float2 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }
FLWCScalar LWCDot(FLWCVector3 Lhs, FLWCVector3 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }
FLWCScalar LWCDot(FLWCVector3 Lhs, float3 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }
FLWCScalar LWCDot(FLWCVector4 Lhs, FLWCVector4 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }
FLWCScalar LWCDot(FLWCVector4 Lhs, float4 Rhs) { return LWCVectorSum(LWCMultiply(Lhs, Rhs)); }

// LWCLength2Scaled returns the length2 of the vector, scaled by the LWC tile size
FLWCScalar LWCLength2Scaled(FLWCScalar V)
{
	return LWCSquareScaled(V);
}

FLWCScalar LWCLength2Scaled(FLWCVector2 V)
{
	FLWCScalar X2 = LWCSquareScaled(LWCGetX(V));
	FLWCScalar Y2 = LWCSquareScaled(LWCGetY(V));
	return LWCAdd(X2, Y2);
}

FLWCScalar LWCLength2Scaled(FLWCVector3 V)
{
	FLWCScalar X2 = LWCSquareScaled(LWCGetX(V));
	FLWCScalar Y2 = LWCSquareScaled(LWCGetY(V));
	FLWCScalar Z2 = LWCSquareScaled(LWCGetZ(V));
	return LWCAdd(LWCAdd(X2, Y2), Z2);
}

FLWCScalar LWCLength2Scaled(FLWCVector4 V)
{
	FLWCScalar X2 = LWCSquareScaled(LWCGetX(V));
	FLWCScalar Y2 = LWCSquareScaled(LWCGetY(V));
	FLWCScalar Z2 = LWCSquareScaled(LWCGetZ(V));
	FLWCScalar W2 = LWCSquareScaled(LWCGetW(V));
	return LWCAdd(LWCAdd(LWCAdd(X2, Y2), Z2), W2);
}

// LWCLength2Scaled scales the result by TileSize, which means the result of LWCSqrtUnscaled needs to be scaled by TileSize as well
// Rather than apply that scale directly, just put the result into the TileCoordinate (with no offset)
FLWCScalar LWCLength(FLWCScalar V) { return MakeLWCScalar(LWCSqrtUnscaled(LWCLength2Scaled(V)), 0.0f); }
FLWCScalar LWCLength(FLWCVector2 V) { return MakeLWCScalar(LWCSqrtUnscaled(LWCLength2Scaled(V)), 0.0f); }
FLWCScalar LWCLength(FLWCVector3 V) { return MakeLWCScalar(LWCSqrtUnscaled(LWCLength2Scaled(V)), 0.0f); }
FLWCScalar LWCLength(FLWCVector4 V) { return MakeLWCScalar(LWCSqrtUnscaled(LWCLength2Scaled(V)), 0.0f); }

float LWCRcpLength(FLWCScalar V) { return LWCRsqrtScaled(LWCLength2Scaled(V), UE_LWC_RENDER_TILE_SIZE_RCP); }
float LWCRcpLength(FLWCVector2 V) { return LWCRsqrtScaled(LWCLength2Scaled(V), UE_LWC_RENDER_TILE_SIZE_RCP); }
float LWCRcpLength(FLWCVector3 V) { return LWCRsqrtScaled(LWCLength2Scaled(V), UE_LWC_RENDER_TILE_SIZE_RCP); }
float LWCRcpLength(FLWCVector4 V) { return LWCRsqrtScaled(LWCLength2Scaled(V), UE_LWC_RENDER_TILE_SIZE_RCP); }

float LWCNormalize(FLWCScalar V) { return 1.0f; } // Normalizing a scalar always results in 1
float2 LWCNormalize(FLWCVector2 V) { return LWCToFloat(LWCMultiply(V, LWCRcpLength(V))); }
float3 LWCNormalize(FLWCVector3 V) { return LWCToFloat(LWCMultiply(V, LWCRcpLength(V))); }
float4 LWCNormalize(FLWCVector4 V) { return LWCToFloat(LWCMultiply(V, LWCRcpLength(V))); }

// Needed for new translator
FLWCScalar LWCDistance(FLWCScalar A, FLWCScalar B) { return LWCAbs(LWCSubtract(A, B)); }
FLWCScalar LWCDistance(FLWCVector2 A, FLWCVector2 B) { return LWCLength(LWCSubtract(A, B)); }
FLWCScalar LWCDistance(FLWCVector3 A, FLWCVector3 B) { return LWCLength(LWCSubtract(A, B)); }
FLWCScalar LWCDistance(FLWCVector4 A, FLWCVector4 B) { return LWCLength(LWCSubtract(A, B)); }

// LWCHackToFloat is a marker for places where LWC quantities are transformed to float, without regard for range/precision
// This will work correctly for content that's not using LWC scale values
// 'LWC_HACK_TO_ZERO' is a big hammer to work around some internal compiler errors....using this will break the functionality of the shader, but can unblock testing of other shaders
#ifdef LWC_HACK_TO_ZERO
float3 LWCHackToFloat(FLWCVector3 v) { return (float3)0.0f; }
float4 LWCHackToFloat(FLWCVector4 v) { return (float4)0.0f; }
float4x4 LWCHackToFloat(FLWCMatrix v) { return (float4x4)0.0f; }
float4x4 LWCHackToFloat(FLWCInverseMatrix v) { return (float4x4)0.0f; }
#else
#define LWCHackToFloat(V) LWCToFloat(V)
#endif

#define DEFINE_DF_TO_TILEOFFSET_OPERATOR(FDFType, FLWCType, LWCConstructor, FFloatType) \
	FLWCType DFToTileOffset(FDFType In)																							\
	{																															\
		/* CPU side defines MakeTile as floor(Value / TILE_SIZE + 0.5) which is not quite the same as round but */				\
		/* divergent cases are extremely rare and result simply in rollover from one tile edge to the edge of the next tile */	\
		/* Values greater than 2^(24+floor(log2(TILE_SIZE-1))) are not converted correctly, because In.Low is assumed to be */	\
		/* less than half a tile. These values by definition exceed WORLD_MAX, so this is not an issue. */						\
		FFloatType Tile = round(In.High / UE_LWC_RENDER_TILE_SIZE);																\
		FFloatType Offset = INVARIANT_ADD(INVARIANT_FMA(Tile, -UE_LWC_RENDER_TILE_SIZE, In.High), In.Low);						\
																																\
		return LWCConstructor(Tile, Offset);																					\
	}	
	
#if UE_DF_NO_FAST_MATH
#define DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFType, FLWCType, LWCConstructor, FFloatType) \
	FLWCType DFFastToTileOffset(FDFType In)																						\
	{																															\
		return DFToTileOffset(In);																								\
	}
#else
#define DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFType, FLWCType, LWCConstructor, FFloatType) \
	FLWCType DFFastToTileOffset(FDFType In)																						\
	{																															\
		/* Produces a non-normalized tile-offset coordinate. */																	\
		/* If we're not rounding, Tile has the full precision of In.High because UE_LWC_RENDER_TILE_SIZE is pow2 */				\
		/* Therefore, we don't need to calculate the error term and can just use In.Low */										\
		 FFloatType Tile = In.High * (1 / UE_LWC_RENDER_TILE_SIZE); 															\
		 FFloatType Offset = In.Low; 																							\
																																\
		return LWCConstructor(Tile, Offset);																					\
	}
#endif

DEFINE_DF_TO_TILEOFFSET_OPERATOR(FDFScalar, FLWCScalar, MakeLWCScalar, float)
DEFINE_DF_TO_TILEOFFSET_OPERATOR(FDFVector2, FLWCVector2, MakeLWCVector2, float2)
DEFINE_DF_TO_TILEOFFSET_OPERATOR(FDFVector3, FLWCVector3, MakeLWCVector3, float3)
DEFINE_DF_TO_TILEOFFSET_OPERATOR(FDFVector4, FLWCVector4, MakeLWCVector4, float4)
DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFScalar, FLWCScalar, MakeLWCScalar, float)
DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFVector2, FLWCVector2, MakeLWCVector2, float2)
DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFVector3, FLWCVector3, MakeLWCVector3, float3)
DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR(FDFVector4, FLWCVector4, MakeLWCVector4, float4)
#undef DEFINE_DF_TO_TILEOFFSET_OPERATOR
#undef DEFINE_DF_FAST_TO_TILEOFFSET_OPERATOR

FLWCMatrix DFToTileOffset(FDFMatrix In)
{
	FLWCVector3 PosHigh = DFToTileOffset(MakeDFVector3(In.PostTranslation, 0));

	float4x4 M = MultiplyTranslation(In.M, PosHigh.Offset);
	float3 Tile = PosHigh.Tile;
	return MakeLWCMatrix(Tile, M);
}

FLWCMatrix DFFastToTileOffset(FDFMatrix In)
{
#if UE_DF_NO_FAST_MATH
	return DFToTileOffset(In);
#else
	float4x4 M = In.M;
	float3 Tile = In.PostTranslation / UE_LWC_RENDER_TILE_SIZE;
	return MakeLWCMatrix(Tile, M);
#endif
}

FLWCInverseMatrix DFToTileOffset(FDFInverseMatrix In)
{
	FLWCVector3 PosHigh = DFToTileOffset(MakeDFVector3(In.PreTranslation, 0));
	float4x4 M = MultiplyTranslation(-PosHigh.Offset, In.M);
	float3 Tile = PosHigh.Tile;
	return MakeLWCInverseMatrix(Tile, M);
}

FLWCInverseMatrix DFFastToTileOffset(FDFInverseMatrix In)
{
#if UE_DF_NO_FAST_MATH
	return DFToTileOffset(In);
#else
	float4x4 M = In.M;
	float3 Tile = In.PreTranslation / UE_LWC_RENDER_TILE_SIZE;
	return MakeLWCInverseMatrix(Tile, M);
#endif
}

#define DFToTileOffset_Hack DFToTileOffset

FDFScalar DFFromTileOffset(FLWCScalar In)
{
	float TileOffset = In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return DFFastTwoSum(TileOffset, In.Offset);
}

FDFVector2 DFFromTileOffset(FLWCVector2 In)
{
	float2 TileOffset = In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return DFFastTwoSum(TileOffset, In.Offset);
}

FDFVector3 DFFromTileOffset(FLWCVector3 In)
{
	float3 TileOffset = In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return DFFastTwoSum(TileOffset, In.Offset);
}

FDFVector4 DFFromTileOffset(FLWCVector4 In)
{
	float4 TileOffset = In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return DFFastTwoSum(TileOffset, In.Offset);
}

FDFMatrix DFFromTileOffset(FLWCMatrix In)
{
	float4x4 M = In.M;
	float3 PostTranslation = In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return MakeDFMatrix(PostTranslation, M);
}

FDFInverseMatrix DFFromTileOffset(FLWCInverseMatrix In)
{
	float4x4 M = In.M;
	float3 PreTranslation = -In.Tile * UE_LWC_RENDER_TILE_SIZE;
	return MakeDFInverseMatrix(PreTranslation, M);
}

#define DFFromTileOffset_Hack DFFromTileOffset

// Hacky LWCFunc(DFParam) shims for custom hlsl nodes in materials
float LWCToFloat(FDFScalar Value)   { return DFDemote(Value); }
float2 LWCToFloat(FDFVector2 Value) { return DFDemote(Value); }
float3 LWCToFloat(FDFVector3 Value) { return DFDemote(Value); }
float4 LWCToFloat(FDFVector4 Value) { return DFDemote(Value); }
float4x4 LWCToFloat(FDFMatrix Value) { return DFDemote(Value); }
float4x4 LWCToFloat(FDFInverseMatrix Value) { return DFDemote(Value); }

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
FLWCMatrix WaveReadLaneAt(FLWCMatrix In, uint SrcIndex)
{
	FLWCMatrix Result;
	Result.M[0] = WaveReadLaneAt(In.M[0], SrcIndex);
	Result.M[1] = WaveReadLaneAt(In.M[1], SrcIndex);
	Result.M[2] = WaveReadLaneAt(In.M[2], SrcIndex);
	Result.M[3] = WaveReadLaneAt(In.M[3], SrcIndex);
	Result.Tile = WaveReadLaneAt(In.Tile, SrcIndex);
	return Result;
}

FLWCInverseMatrix WaveReadLaneAt(FLWCInverseMatrix In, uint SrcIndex)
{
	FLWCInverseMatrix Result;
	Result.M[0] = WaveReadLaneAt(In.M[0], SrcIndex);
	Result.M[1] = WaveReadLaneAt(In.M[1], SrcIndex);
	Result.M[2] = WaveReadLaneAt(In.M[2], SrcIndex);
	Result.M[3] = WaveReadLaneAt(In.M[3], SrcIndex);
	Result.Tile = WaveReadLaneAt(In.Tile, SrcIndex);
	return Result;
}
#endif