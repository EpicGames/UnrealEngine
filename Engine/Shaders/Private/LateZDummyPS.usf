// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Common.ush"

void MainPS(float4 SVPos : SV_Position, out float OutDepth : SV_Depth)
{
	OutDepth = SVPos.z;
}