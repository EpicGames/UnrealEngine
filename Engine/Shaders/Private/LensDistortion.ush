// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"


float2 ApplyLensDistortionOnViewportUV(Texture2D<float2> DisplacementTexture, SamplerState DisplacementSampler, float2 DestViewportUV)
{
	return DestViewportUV + DisplacementTexture.SampleLevel(DisplacementSampler, DestViewportUV, 0);
}

float2 ApplyLensDistortionOnScreenPos(Texture2D<float2> DisplacementTexture, SamplerState DisplacementSampler, float2 DestScreenPos)
{
	float2 DestViewportUV = ScreenPosToViewportUV(DestScreenPos);

	return DestScreenPos + DisplacementTexture.SampleLevel(DisplacementSampler, DestViewportUV, 0) * float2(2, -2);
}

#if SUPPORTS_INDEPENDENT_SAMPLERS

float2 ApplyLensDistortionOnViewportUV(Texture2D<float2> DisplacementTexture, float2 DestViewportUV)
{
	return ApplyLensDistortionOnViewportUV(DisplacementTexture, GlobalBilinearClampedSampler, DestViewportUV);
}

float2 ApplyLensDistortionOnScreenPos(Texture2D<float2> DisplacementTexture, float2 DestScreenPos)
{
	return ApplyLensDistortionOnScreenPos(DisplacementTexture, GlobalBilinearClampedSampler, DestScreenPos);
}

#endif
