// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightAccumulator.usf: FLightAccumulator "class" and it's methods, useful for screen space subsurface scattering
=============================================================================*/

#pragma once

#include "Common.ush"

// set by c++, not set for LPV
// 0 / 1
#ifndef VISUALIZE_LIGHT_CULLING
	#define VISUALIZE_LIGHT_CULLING 0
#endif
 
// for ScreenSpaceSubsurfaceScattering
// 0 : fastest (speculars leak in SSSSS). Forces checkerboard rendering for subsurface profile
// 1 : luminance specular - works well (needs 64bit SceneColor, stores luminance of specular in alpha, can be optimized further in LightAccumulator_Add() ) 
//	   NOTE: Dependent on View.bCheckerboardSubsurfaceProfileRendering - if this is off, we fall back to mode 0
// 2 : colored specular works well (best, requires another RT for light accumulation)
#define SUBSURFACE_CHANNEL_MODE 1

struct FLightAccumulator
{
	float3 TotalLight;
	float TotalLightLuminance;

	// only actually used SUBSURFACE_CHANNEL_MODE == 1
	// assumed to be compiled out otherwise (not compiled out with #if so we can use if() instead of #if for better readability and compiler error checking)
	// input for ScreenSpaceSubsurfaceScattering
	float ScatterableLightLuma;

	// only actually used SUBSURFACE_CHANNEL_MODE == 2
	// assumed to be compiled out otherwise (not compiled out with #if so we can use if() instead of #if for better readability and compiler error checking)
	// input for ScreenSpaceSubsurfaceScattering
	float3 ScatterableLight;

	// only used for development (not compiled out with #if so we can use if() instead of #if for better readability and compiler error checking)
	// assumed to be compiled out otherwise
	float EstimatedCost;

	// only used for alpha, which needs to keep specular and alpha separate since specular needs to multiply by 1/opacity to compensate for alpha blending
	// assumed to be compiled out otherwise
	float3 TotalLightDiffuse;
	float3 TotalLightSpecular;

};

struct FDeferredLightingSplit
{
	float4 DiffuseLighting;
	float4 SpecularLighting;
	float LightingLuminance;
};

// accumulate light, can be called multiple times
void LightAccumulator_AddSplit(inout FLightAccumulator In, float3 DiffuseTotalLight, float3 SpecularTotalLight, float3 ScatterableLight, float3 CommonMultiplier, const bool bNeedsSeparateSubsurfaceLightAccumulation)
{
	// 3 mad
	In.TotalLight += (DiffuseTotalLight + SpecularTotalLight) * CommonMultiplier;
	In.TotalLightLuminance += Luminance((DiffuseTotalLight + SpecularTotalLight) * CommonMultiplier);

	// This should ideally be evaluated statically outside of this function to avoid the branch
	if (bNeedsSeparateSubsurfaceLightAccumulation)
	{
		if (SUBSURFACE_CHANNEL_MODE == 1)
		{
			if (View.bCheckerboardSubsurfaceProfileRendering == 0)
			{
				In.ScatterableLightLuma += Luminance(ScatterableLight * CommonMultiplier);
			}
		}
		else if (SUBSURFACE_CHANNEL_MODE == 2)
		{
			// 3 mad
			In.ScatterableLight += ScatterableLight * CommonMultiplier;
		}
	}

	In.TotalLightDiffuse += DiffuseTotalLight * CommonMultiplier;
	In.TotalLightSpecular += SpecularTotalLight * CommonMultiplier;
}

void LightAccumulator_Add(inout FLightAccumulator In, float3 TotalLight, float3 ScatterableLight, float3 CommonMultiplier, const bool bNeedsSeparateSubsurfaceLightAccumulation)
{
	LightAccumulator_AddSplit(In, TotalLight, 0.0f, ScatterableLight, CommonMultiplier, bNeedsSeparateSubsurfaceLightAccumulation);
}

float4 ConvertEstimatedCostToColor(float EstimatedCost)
{
	return 0.1f * float4(1.0f, 0.25f, 0.075f, 0) * EstimatedCost;
}

//
// compute final value to store in the MRT0
// @retrun RGB:SceneColor Specular and Diffuse, A:Non Specular SceneColor Luminance
float4 LightAccumulator_GetResult(FLightAccumulator In)
{
	float4 Ret;

	if (VISUALIZE_LIGHT_CULLING == 1)
	{
		// a soft gradient from dark red to bright white, can be changed to be different
		Ret = ConvertEstimatedCostToColor(In.EstimatedCost);
	}
	else
	{
		Ret = float4(In.TotalLight, 0);

		if (SUBSURFACE_CHANNEL_MODE == 1 )
		{
			// bSubsurfacePostprocessEnabled	bCheckerboardSubsurfaceProfileRendering  OutputLuma
			// 0								0											0
			// 0								1											0
			// 1								0											1
			// 1								1											0
			// The alpha channel will not be used in the post process subsurface pass if View.bSubsurfacePostprocessEnabled
			// is not enabled. Add this gate (View.bSubsurfacePostprocessEnabled) so that when subsurface is disabled, scene 
			// captures will not pick up ScatterableLightLuma to pollute the alpha channel.
			if (View.bCheckerboardSubsurfaceProfileRendering == 0 && View.bSubsurfacePostprocessEnabled)
			{
				// RGB accumulated RGB HDR color, A: specular luminance for screenspace subsurface scattering
				Ret.a = In.ScatterableLightLuma;
			}
		}
		else if (SUBSURFACE_CHANNEL_MODE == 2)
		{
			// RGB accumulated RGB HDR color, A: view independent (diffuse) luminance for screenspace subsurface scattering
			// 3 add,  1 mul, 2 mad, can be optimized to use 2 less temporary during accumulation and remove the 3 add
			Ret.a = Luminance(In.ScatterableLight);
			// todo, need second MRT for SUBSURFACE_CHANNEL_MODE==2
		}
	}

	return Ret;
}


FDeferredLightingSplit LightAccumulator_GetResultSplit(FLightAccumulator In)
{
	float4 RetDiffuse;
	float4 RetSpecular;

	if (VISUALIZE_LIGHT_CULLING == 1)
	{
		// a soft gradient from dark red to bright white, can be changed to be different
		RetDiffuse  = ConvertEstimatedCostToColor(In.EstimatedCost);
		RetSpecular = RetDiffuse;
	}
	else
	{
		RetDiffuse = float4(In.TotalLightDiffuse, 0);
		RetSpecular = float4(In.TotalLightSpecular, 0);

		if (SUBSURFACE_CHANNEL_MODE == 1 )
		{
			if (View.bCheckerboardSubsurfaceProfileRendering == 0 && View.bSubsurfacePostprocessEnabled)
			{
				// RGB accumulated RGB HDR color, A: specular luminance for screenspace subsurface scattering
				RetDiffuse.a = In.ScatterableLightLuma;
			}
		}
		else if (SUBSURFACE_CHANNEL_MODE == 2)
		{
			// RGB accumulated RGB HDR color, A: view independent (diffuse) luminance for screenspace subsurface scattering
			// 3 add,  1 mul, 2 mad, can be optimized to use 2 less temporary during accumulation and remove the 3 add
			RetDiffuse.a = Luminance(In.ScatterableLight);
			// todo, need second MRT for SUBSURFACE_CHANNEL_MODE==2
		}
	}

	FDeferredLightingSplit Ret;
	Ret.DiffuseLighting = RetDiffuse;
	Ret.SpecularLighting = RetSpecular;
	Ret.LightingLuminance = In.TotalLightLuminance;

	return Ret;
}

struct FDeferredLightingToonData
{
	float3 DiffuseLuminance;
	float Weight;
	float ShadowNoL;
};

struct FSubstrateDeferredLighting
{
	float4 SceneColor;						// Untouched scene luminance, alpha channel may contain scatterable light luminance.
	FDeferredLightingToonData ToonData;
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	float3 OpaqueRoughRefractionSceneColor;	// Luminance that is going to be blurier by the top layer roughness
	float3 SubSurfaceSceneColor;			// Scene color that should go through the sub surface post process. There is not alpha: not needed because it is separated from specular already.
#endif

	// only used for development (not compiled out with #if so we can use if() instead of #if for better readability and compiler error checking)
	// assumed to be compiled out otherwise
	float EstimatedCost;

	float3 TotalDiffuseLighting;
	float3 TotalSpecularLighting;
};

FSubstrateDeferredLighting GetInitialisedSubstrateDeferredLighting()
{
	FSubstrateDeferredLighting Result = (FSubstrateDeferredLighting)0;
	return Result;
}

void AccumulateSubstrateDeferredLighting(inout FSubstrateDeferredLighting SubstrateLighting, FLightAccumulator In, bool bDiffuseIsSubsurface, bool bIsToplayer)
{
	FDeferredLightingSplit DiffSpec = LightAccumulator_GetResultSplit(In);

#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	if (bIsToplayer)
	{
		SubstrateLighting.SceneColor						+= DiffSpec.SpecularLighting + (bDiffuseIsSubsurface ? 0 : DiffSpec.DiffuseLighting);
		SubstrateLighting.OpaqueRoughRefractionSceneColor	+= 0.0;
		SubstrateLighting.SubSurfaceSceneColor				+=(bDiffuseIsSubsurface ? DiffSpec.DiffuseLighting : 0).rgb;
	}
	else // !bIsToplayer
	{
		SubstrateLighting.SceneColor						+= 0.0;
		SubstrateLighting.OpaqueRoughRefractionSceneColor	+=(DiffSpec.SpecularLighting +(bDiffuseIsSubsurface ? 0 : DiffSpec.DiffuseLighting)).rgb;
		SubstrateLighting.SubSurfaceSceneColor				+=(bDiffuseIsSubsurface ? DiffSpec.DiffuseLighting : 0).rgb;
	}
#else
	SubstrateLighting.SceneColor += DiffSpec.DiffuseLighting + DiffSpec.SpecularLighting;
#endif

	SubstrateLighting.EstimatedCost += In.EstimatedCost;
}
