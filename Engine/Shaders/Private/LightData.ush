// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "HairShadingCommon.ush" // Used for FHairTransmittanceData
#include "/Engine/Shared/LightViewData.h"
#include "/Engine/Shared/LightDefinitions.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Defines

#define LIGHTING_CHANNEL_MASK		0x7

#define MAX_RECT_ATLAS_MIP			32

// Capsule lights are supported by default in mobile deferred shading (including translucency), and are disabled in mobile forward
#ifndef MOBILE_SHADING_PATH_SUPPORT_CAPSULE_LIGHT
#define MOBILE_SHADING_PATH_SUPPORT_CAPSULE_LIGHT MOBILE_DEFERRED_SHADING
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Rect light texture data
struct FRectTexture
{
	half2 AtlasUVOffset;
	half2 AtlasUVScale;
	half  AtlasMaxLevel;
};

FRectTexture InitRectTexture()
{
	FRectTexture Out;
	Out.AtlasUVOffset = 0;
	Out.AtlasUVScale  = 0;
	Out.AtlasMaxLevel = MAX_RECT_ATLAS_MIP;
	return Out;
}

struct FRectLightData
{
	float		 BarnCosAngle;
	float		 BarnLength;
	FRectTexture AtlasData;
};

FRectLightData UnpackRectLightData(uint In0, uint In1, uint In2)
{
	FRectLightData Out;
	Out.AtlasData.AtlasUVOffset	= half2(f16tof32(In0 & 0xFFFF), f16tof32(In0 >> 16));
	Out.AtlasData.AtlasUVScale	= half2(f16tof32(In1 & 0xFFFF), f16tof32(In1 >> 16));
	Out.BarnLength				= f16tof32(In2 & 0xFFFF);			// 16 bits
	Out.BarnCosAngle			= UnpackUnorm10(In2 >> 16);			// 10 bits
	Out.AtlasData.AtlasMaxLevel = half(In2 >> 26);					//  6 bits
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Directional light data

struct FDirectionalLightData
{
	bool   HasDirectionalLight;
	uint   DirectionalLightSceneInfoExtraDataPacked;
	float2 DirectionalLightDistanceFadeMAD;
	float3 DirectionalLightColor;
	float3 DirectionalLightDirection;
	float  DirectionalLightSourceRadius;
	float  DirectionalLightSoftSourceRadius;
	float  DirectionalLightSpecularScale;
	float  DirectionalLightDiffuseScale;
	uint  LightFunctionAtlasLightIndex;
	bool  bAffectsTranslucentLighting;
	bool  bHandledByMegaLights;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Forward light data
struct FForwardLightData
{
	// Raw value form light buffers
	float4 LightPositionAndInvRadius;
	float4 LightColorAndIdAndFalloffExponentAndRayEndBias;
	float4 SpotAnglesAndSourceRadiusPacked;
	float4 LightDirectionAndSceneInfoExtraDataPacked;
	float4 LightTangentAndIESDataAndSpecularScale;
	float3 RectData;

	// Derived terms after unpacking
	/** Virtual shadow map ID or INDEX_NONE if not present.*/
	int  VirtualShadowMapId;
	/** Previous frame LocalLightIndex or INDEX_NONE if not present.*/
	int  PrevLocalLightIndex;
	/** Index into GPUScene lights */
	int LightSceneId;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper struct to provide type safety when loading lights from the Light Grid
// Local lights can optimize bRadialLight=true
struct FLocalLightData
{
	FForwardLightData Internal;
#if !COMPILER_SUPPORTS_HLSL2021
	uint Unused; // necessary to avoid ambiguity with struct FForwardLightData
#endif
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Simple light data

/** Data about a single light to be shaded with the simple shading model, designed for speed and limited feature set. */
struct FSimpleDeferredLightData
{
	float3 TranslatedWorldPosition;
	float  InvRadius;
	float3 Color;
	float  FalloffExponent;
	/** Whether to use inverse squared falloff. */
	bool bInverseSquared;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
 // Light-weight light data - Used for raytracing/VSM/Lumen/...
 
// Mirrors FLightShaderParameters on C++ side.
struct FLightShaderParameters
{
	float3 TranslatedWorldPosition;
	float  InvRadius;
	float3 Color;
	float  FalloffExponent;
	float3 Direction;
	float3 Tangent;
	float2 SpotAngles;
	float  SpecularScale;
	float  DiffuseScale;
	float  SourceRadius;
	float  SoftSourceRadius;
	float  SourceLength;
	float  RectLightBarnCosAngle;
	float  RectLightBarnLength;
	float2 RectLightAtlasUVOffset;
	float2 RectLightAtlasUVScale;
	float  RectLightAtlasMaxLevel;
	float  IESAtlasIndex;
	uint  LightFunctionAtlasLightIndex;
	uint  bAffectsTranslucentLighting;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Heavy light data - used for deferred lighting functions

/**
 * Data about a single light.
 * Putting the light data in this struct allows the same lighting code to be used between standard deferred,
 * Where many light properties are known at compile time, and tiled deferred, where all light properties have to be fetched from a buffer.
 */
 // TODO: inherit or compose FLightShaderParameters
struct FDeferredLightData
{
	float3 TranslatedWorldPosition;
	half   InvRadius;
	/** Premultiplied with light brightness, can be huge. Do not use half precision here */
	float3 Color;
	half   FalloffExponent;
	float3 Direction;
	float3 Tangent;
	float  SoftSourceRadius;
	half2  SpotAngles;
	float  SourceRadius;
	float  SourceLength;
	half   SpecularScale;
	half   DiffuseScale;
	float  ContactShadowLength;
	/** Intensity of shadow-casting contact shadows */
	float ContactShadowCastingIntensity;
	/** Intensity of non-shadow-casting contact shadows */
	float ContactShadowNonCastingIntensity;
	float2 DistanceFadeMAD;
	half4  ShadowMapChannelMask;
	/** Whether ContactShadowLength is in World Space or in Screen Space. */
	bool   ContactShadowLengthInWS;
	/** Whether to use inverse squared falloff. */
	bool   bInverseSquared;
	/** Whether this is a light with radial attenuation, aka point or spot light. */
	bool   bRadialLight;
	/** Whether this light needs spotlight attenuation. */
	bool   bSpotLight;
	bool   bRectLight;
	/** Whether the light should apply shadowing. */
	uint   ShadowedBits;
	/** Rect light data. */
	FRectLightData RectLightData;
	/** IES data. */
	float  IESAtlasIndex;
	// Light function atlas index 
	uint  LightFunctionAtlasLightIndex;
	/** Hair transmittance data. */
	FHairTransmittanceData HairTransmittance;
	// Wether the light affect translucent materials or not 
	uint  bAffectsTranslucentLighting;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Pack/unpack function related to light structures

float3 UnpackLightTranslatedWorldPosition(FForwardLightData In)
{
	return In.LightPositionAndInvRadius.xyz;
}

float3 UnpackLightTranslatedWorldPosition(FLocalLightData In)
{
	return UnpackLightTranslatedWorldPosition(In.Internal);
}

float UnpackLightInvRadius(FForwardLightData In)
{
	return In.LightPositionAndInvRadius.w;
}

float UnpackLightInvRadius(FLocalLightData In)
{
	return UnpackLightInvRadius(In.Internal);
}

float3 UnpackLightDirection(FForwardLightData In)
{
	return In.LightDirectionAndSceneInfoExtraDataPacked.xyz;
}

float3 UnpackLightDirection(FLocalLightData In)
{
	return UnpackLightDirection(In.Internal);
}

uint UnpackLightSceneInfoExtraDataPacked(FForwardLightData In)
{
	return asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
}

uint UnpackLightSceneInfoExtraDataPacked(FLocalLightData In)
{
	return UnpackLightSceneInfoExtraDataPacked(In.Internal);
}

float3 UnpackLightColor(float2 In)
{
	float3 Dir = float3(
		((asuint(In.y) >> 0) & 0x3FF),
		((asuint(In.y) >> 10) & 0x3FF),
		((asuint(In.y) >> 20) & 0x3FF));
	return In.x * Dir;
}

float3 UnpackLightColor(FForwardLightData In)
{
	return UnpackLightColor(In.LightColorAndIdAndFalloffExponentAndRayEndBias.xy);
}

float3 UnpackLightColor(FLocalLightData In)
{
	return UnpackLightColor(In.Internal);
}

float UnpackLightFalloffExponent(FForwardLightData In)
{
	return f16tof32(asuint(In.LightColorAndIdAndFalloffExponentAndRayEndBias.w) & 0xFFFF);
}

float UnpackLightFalloffExponent(FLocalLightData In)
{
	return UnpackLightFalloffExponent(In.Internal);
}

float UnpackLightRayEndBias(FForwardLightData In)
{
	return f16tof32(asuint(In.LightColorAndIdAndFalloffExponentAndRayEndBias.w) >> 16);
}

float UnpackLightRayEndBias(FLocalLightData In)
{
	return UnpackLightRayEndBias(In.Internal);
}

float4 UnpackShadowMapChannelMask(uint In)
{
	// 8 ALU
	return float4((In & 0x1), (In & 0x2) >> 1, (In & 0x4) >> 2, (In & 0x8) >> 3);
}

uint UnpackLightingChannelMask(uint InLightSceneInfoExtraDataPacked)
{
	return (InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHTING_CHANNEL_MASK) & LIGHTING_CHANNEL_MASK;
}

uint UnpackLightingChannelMask(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackLightingChannelMask(LightSceneInfoExtraDataPacked);
}

uint UnpackLightingChannelMask(FLocalLightData In)
{
	return UnpackLightingChannelMask(In.Internal);
}

uint UnpackLightType(uint InLightSceneInfoExtraDataPacked)
{
	return (InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_TYPE) & 0x3;
}

bool UnpackCastShadow(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_SHADOW) & 0x1) != 0;
}

bool UnpackCastShadow(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackCastShadow(LightSceneInfoExtraDataPacked);
}

bool UnpackCastShadow(FLocalLightData In)
{
	return UnpackCastShadow(In.Internal);
}

bool UnpackCastVolumetricShadow(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_VOLUMETRIC_SHADOWS) & 0x1) != 0;
}

bool UnpackCastVolumetricShadow(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackCastVolumetricShadow(LightSceneInfoExtraDataPacked);
}

bool UnpackCastVolumetricShadow(FLocalLightData In)
{
	return UnpackCastVolumetricShadow(In.Internal);
}

bool UnpackUseTransmission(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_USE_TRANSMISSION) & 0x1) != 0;
}

bool UnpackUseTransmission(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackUseTransmission(LightSceneInfoExtraDataPacked);
}

bool UnpackHasLightFunction(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_HAS_LIGHT_FUNCTION) & 0x1) != 0;
}

bool UnpackHasLightFunction(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackHasLightFunction(LightSceneInfoExtraDataPacked);
}

bool UnpackHasLightFunction(FLocalLightData In)
{
	return UnpackHasLightFunction(In.Internal);
}

uint UnpackLightFunctionAtlasIndex(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_FUNCTION_ATLAS_LIGHT_INDEX) & 0xFF);
}

uint UnpackLightFunctionAtlasIndex(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackLightFunctionAtlasIndex(LightSceneInfoExtraDataPacked);
}

uint UnpackLightFunctionAtlasIndex(FLocalLightData In)
{
	return UnpackLightFunctionAtlasIndex(In.Internal);
}

bool UnpackAffectsTranslucentLighting(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_AFFECT_TRANSLUCENT_LIGHTING) & 0x1) != 0;
}

bool UnpackAffectsTranslucentLighting(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackAffectsTranslucentLighting(LightSceneInfoExtraDataPacked);
}

bool UnpackAffectsTranslucentLighting(FLocalLightData In)
{
	return UnpackAffectsTranslucentLighting(In.Internal);
}

uint UnpackIsHandledByMegaLights(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_MEGA_LIGHT) & 0x1);
}

uint UnpackIsHandledByMegaLights(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackIsHandledByMegaLights(LightSceneInfoExtraDataPacked);
}

uint UnpackIsHandledByMegaLights(FLocalLightData In)
{
	return UnpackIsHandledByMegaLights(In.Internal);
}

uint UnpackIsClusteredDeferredSupported(uint InLightSceneInfoExtraDataPacked)
{
	return ((InLightSceneInfoExtraDataPacked >> LIGHT_EXTRA_DATA_BIT_OFFSET_CLUSTERED_DEFERRED_SUPPORTED) & 0x1);
}

uint UnpackIsClusteredDeferredSupported(FForwardLightData In)
{
	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	return UnpackIsClusteredDeferredSupported(LightSceneInfoExtraDataPacked);
}

uint UnpackIsClusteredDeferredSupported(FLocalLightData In)
{
	return UnpackIsClusteredDeferredSupported(In.Internal);
}

float UnpackVolumetricScatteringIntensity(FForwardLightData In)
{
	return f16tof32(asuint(In.SpotAnglesAndSourceRadiusPacked.w) >> 16);
}

float UnpackVolumetricScatteringIntensity(FLocalLightData In)
{
	return UnpackVolumetricScatteringIntensity(In.Internal);
}

float UnpackLightSourceRadius(FForwardLightData In)
{
	return f16tof32(asuint(In.SpotAnglesAndSourceRadiusPacked.z) & 0xFFFF);
}

float UnpackLightSourceRadius(FLocalLightData In)
{
	return UnpackLightSourceRadius(In.Internal);
}

float UnpackLightSoftSourceRadius(FForwardLightData In)
{
	return f16tof32(asuint(In.SpotAnglesAndSourceRadiusPacked.z) >> 16);
}

float UnpackLightSoftSourceRadius(FLocalLightData In)
{
	return UnpackLightSoftSourceRadius(In.Internal);
}

float UnpackLightSourceLength(FForwardLightData In)
{
	return f16tof32(asuint(In.SpotAnglesAndSourceRadiusPacked.w));
}

float UnpackLightSourceLength(FLocalLightData In)
{
	return UnpackLightSourceLength(In.Internal);
}

float GetLightSourceRadius(FForwardLightData In)
{	
	return UnpackLightSourceRadius(In);
}

float GetLightSourceRadius(FLocalLightData In)
{
	return GetLightSourceRadius(In.Internal);
}

float2 GetLightSpotAngles(FForwardLightData In)
{
	return In.SpotAnglesAndSourceRadiusPacked.xy;
}

float2 GetLightSpotAngles(FLocalLightData In)
{
	return GetLightSpotAngles(In.Internal);
}

float UnpackLightSpecularScale(FForwardLightData In)
{
	float SpecularScale = UnpackUnorm10(asuint(In.LightTangentAndIESDataAndSpecularScale.w));
	bool bRcpEncoded = (asuint(In.LightTangentAndIESDataAndSpecularScale.w) & 0x40000000) != 0;
	return bRcpEncoded ? rcp(SpecularScale) : SpecularScale;
}

float UnpackLightSpecularScale(FLocalLightData In)
{
	return UnpackLightSpecularScale(In.Internal);
}

float UnpackLightDiffuseScale(FForwardLightData In)
{
	float DiffuseScale = UnpackUnorm10(asuint(In.LightTangentAndIESDataAndSpecularScale.w)>>10);
	bool bRcpEncoded = (asuint(In.LightTangentAndIESDataAndSpecularScale.w) & 0x80000000) != 0;
	return bRcpEncoded ? rcp(DiffuseScale) : DiffuseScale;
}

float UnpackLightDiffuseScale(FLocalLightData In)
{
	return UnpackLightDiffuseScale(In.Internal);
}

float UnpackLightIESAtlasIndex(FForwardLightData In)
{
	// Offset IESAtlasIndex here in order to preserve INDEX_NONE = -1 after encoding
	return int(BitFieldExtractU32(asuint(In.LightTangentAndIESDataAndSpecularScale.w), 10, 20)) - 1;
}

float UnpackLightIESAtlasIndex(FLocalLightData In)
{
	return UnpackLightIESAtlasIndex(In.Internal);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Conversion functions
 
FRectTexture ConvertToRectTexture(FDeferredLightData In)
{
	FRectTexture Output;
	Output.AtlasUVOffset = In.RectLightData.AtlasData.AtlasUVOffset;
	Output.AtlasUVScale  = In.RectLightData.AtlasData.AtlasUVScale;
	Output.AtlasMaxLevel = In.RectLightData.AtlasData.AtlasMaxLevel;
	return Output;
}

void MobileOverrideDeferredLightParameters(inout FDeferredLightData Out)
{
#if	SHADING_PATH_MOBILE
	#if RECT_LIGHT_AS_SPOTLIGHT && !MOBILE_DEFERRED_LIGHTING
		// Render RectLight approximately as SpotLight. This simplification only applied to a forward shading
		Out.bSpotLight = Out.bSpotLight || Out.bRectLight;
		Out.bRectLight = false;
	#endif
		// Override values for performance on mobile
	#if !MOBILE_SHADING_PATH_SUPPORT_CAPSULE_LIGHT
		Out.SpecularScale = 1.0; // If enabled -> UnpackLightSpecularScale(In);
		Out.DiffuseScale = 1.0; // If enabled -> UnpackLightDiffuseScale(In);
		
		if (!Out.bRectLight)
		{
			Out.Tangent = 0;
			Out.SourceRadius = 0;
			Out.SourceLength = 0;
			Out.SoftSourceRadius = 0;
		}
	#endif
#endif
}

void MobileOverrideDeferredLightParameters(inout FLightShaderParameters Out)
{
#if	SHADING_PATH_MOBILE && !MOBILE_SHADING_PATH_SUPPORT_CAPSULE_LIGHT
		Out.Tangent = 0;
		Out.SourceRadius = 0;
		Out.SourceLength = 0;
		Out.SoftSourceRadius = 0;
		Out.SpecularScale = 1.0; // If enabled -> UnpackLightSpecularScale(In);
		Out.DiffuseScale = 1.0; // If enabled -> UnpackLightDiffuseScale(In);
#endif
}

// Helper function for converting FForwardLightData into FDeferredLightData
FDeferredLightData ConvertToDeferredLight(
	const FForwardLightData In, 
	float InSpecularScale, 
	inout half4 OutPreviewShadowMapChannelMask,
	inout uint  OutLightingChannelMask)
{
	FDeferredLightData Out = (FDeferredLightData)0;

	const uint LightSceneInfoExtraDataPacked = asuint(In.LightDirectionAndSceneInfoExtraDataPacked.w);
	const uint LightType = UnpackLightType(LightSceneInfoExtraDataPacked);

	Out.TranslatedWorldPosition = In.LightPositionAndInvRadius.xyz;
	Out.InvRadius				= In.LightPositionAndInvRadius.w;
	Out.Color					= UnpackLightColor(In);
	Out.FalloffExponent			= UnpackLightFalloffExponent(In);
	Out.Direction				= In.LightDirectionAndSceneInfoExtraDataPacked.xyz;
	Out.SpotAngles				= GetLightSpotAngles(In);
	Out.SourceRadius			= UnpackLightSourceRadius(In);
	Out.SourceLength			= UnpackLightSourceLength(In);
	Out.Tangent					= In.LightTangentAndIESDataAndSpecularScale.xyz;
	Out.SoftSourceRadius		= UnpackLightSoftSourceRadius(In);
	Out.bInverseSquared			= Out.FalloffExponent == 0 && (LightType != LIGHT_TYPE_DIRECTIONAL); // Directional lights don't use 'inverse squared attenuation'
	Out.SpecularScale			= UnpackLightSpecularScale(In) * InSpecularScale;
	Out.DiffuseScale			= UnpackLightDiffuseScale(In);
	Out.bRadialLight			= LightType != LIGHT_TYPE_DIRECTIONAL;
	Out.bSpotLight				= LightType == LIGHT_TYPE_SPOT;
	Out.bRectLight				= LightType == LIGHT_TYPE_RECT;
	Out.HairTransmittance		= InitHairTransmittanceData();
	Out.RectLightData			= UnpackRectLightData(
									asuint(In.RectData.x),
									asuint(In.RectData.y),
									asuint(In.RectData.z));
	Out.IESAtlasIndex			= UnpackLightIESAtlasIndex(In);
	Out.LightFunctionAtlasLightIndex = UnpackLightFunctionAtlasIndex(In);

	Out.bAffectsTranslucentLighting = UnpackAffectsTranslucentLighting(In);

	// Shadow
	// * Static shadowing uses ShadowMapChannel, 
	// * Dynamic shadows are packed into light attenuation using PreviewShadowMapChannel
	// Data Layout on [10:0] with:
	// * ShadowMapChannelMask	= [3:0]
	// * PreviewShadowMapChannel= [7:4] 
	// * LightincChannelMask    = [8:10] 
	Out.ShadowedBits				= (LightSceneInfoExtraDataPacked & 0xFF) != 0 ? 1 : 0;
	Out.ShadowMapChannelMask		= UnpackShadowMapChannelMask(LightSceneInfoExtraDataPacked);
	OutPreviewShadowMapChannelMask	= UnpackShadowMapChannelMask(LightSceneInfoExtraDataPacked >> 4);
	OutLightingChannelMask			= UnpackLightingChannelMask(LightSceneInfoExtraDataPacked);

	MobileOverrideDeferredLightParameters(Out);

	return Out;
}

FDeferredLightData ConvertToDeferredLight(
	const FLocalLightData In, 
	float InSpecularScale, 
	inout half4 OutPreviewShadowMapChannelMask,
	inout uint  OutLightingChannelMask)
{
	FDeferredLightData DeferredLightData = ConvertToDeferredLight(In.Internal, InSpecularScale, OutPreviewShadowMapChannelMask, OutLightingChannelMask);
	DeferredLightData.bRadialLight = true;
	DeferredLightData.bInverseSquared = DeferredLightData.FalloffExponent == 0;

	return DeferredLightData;
}

FDeferredLightData ConvertToDeferredLight(const FForwardLightData In)
{
	half4 OutPreviewShadowMapChannelMask = 0;
	uint OutLightingChannelMask = LIGHTING_CHANNEL_MASK;
	return ConvertToDeferredLight(In, 1.0f, OutPreviewShadowMapChannelMask, OutLightingChannelMask);
}

FDeferredLightData ConvertToDeferredLight(const FLocalLightData In)
{
	half4 OutPreviewShadowMapChannelMask = 0;
	uint OutLightingChannelMask = LIGHTING_CHANNEL_MASK;
	return ConvertToDeferredLight(In, 1.0f, OutPreviewShadowMapChannelMask, OutLightingChannelMask);
}

FLightShaderParameters ConvertToLightShaderParameters(FDeferredLightData In)
{
	FLightShaderParameters Out;
	Out.TranslatedWorldPosition	= In.TranslatedWorldPosition;
	Out.InvRadius				= In.InvRadius;
	Out.Color					= In.Color;
	Out.FalloffExponent			= In.FalloffExponent;
	Out.Direction				= In.Direction;
	Out.Tangent					= In.Tangent;
	Out.SpotAngles				= In.SpotAngles;
	Out.SpecularScale			= 1;
	Out.DiffuseScale			= 1;
	Out.SourceRadius			= In.SourceRadius;
	Out.SoftSourceRadius		= In.SoftSourceRadius;
	Out.SourceLength			= In.SourceLength;
	Out.RectLightBarnCosAngle	= In.RectLightData.BarnCosAngle;
	Out.RectLightBarnLength		= In.RectLightData.BarnLength;
	Out.RectLightAtlasUVOffset	= In.RectLightData.AtlasData.AtlasUVOffset;
	Out.RectLightAtlasUVScale	= In.RectLightData.AtlasData.AtlasUVScale;
	Out.RectLightAtlasMaxLevel	= In.RectLightData.AtlasData.AtlasMaxLevel;
	return Out;
}

FLightShaderParameters ConvertFromForward(const FForwardLightData In)
{
	FLightShaderParameters Out	= (FLightShaderParameters)0;
	Out.TranslatedWorldPosition	= In.LightPositionAndInvRadius.xyz;
	Out.InvRadius				= In.LightPositionAndInvRadius.w;
	Out.Color					= UnpackLightColor(In);
	Out.FalloffExponent			= UnpackLightFalloffExponent(In);
	Out.Direction				= In.LightDirectionAndSceneInfoExtraDataPacked.xyz;
	Out.Tangent					= In.LightTangentAndIESDataAndSpecularScale.xyz;
	Out.SpotAngles				= GetLightSpotAngles(In);
	Out.SpecularScale			= UnpackLightSpecularScale(In);
	Out.DiffuseScale			= UnpackLightDiffuseScale(In);
	Out.IESAtlasIndex			= UnpackLightIESAtlasIndex(In);
	Out.SourceRadius			= UnpackLightSourceRadius(In);
	Out.SoftSourceRadius		= UnpackLightSoftSourceRadius(In);
	Out.SourceLength			= UnpackLightSourceLength(In);

	Out.LightFunctionAtlasLightIndex = UnpackLightFunctionAtlasIndex(In);

	Out.bAffectsTranslucentLighting = UnpackAffectsTranslucentLighting(In);

	const FRectLightData RectLightData = UnpackRectLightData(
									asuint(In.RectData.x),
									asuint(In.RectData.y),
									asuint(In.RectData.z));
	Out.RectLightBarnCosAngle	= RectLightData.BarnCosAngle;
	Out.RectLightBarnLength		= RectLightData.BarnLength;
	Out.RectLightAtlasUVOffset	= RectLightData.AtlasData.AtlasUVOffset;
	Out.RectLightAtlasUVScale	= RectLightData.AtlasData.AtlasUVScale;
	Out.RectLightAtlasMaxLevel	= RectLightData.AtlasData.AtlasMaxLevel;

	MobileOverrideDeferredLightParameters(Out);

	return Out;
}

FLightShaderParameters ConvertFromLocal(const FLocalLightData In)
{
	return ConvertFromForward(In.Internal);
}

// Helper function for converting FDirectionalLightData into FDeferredLightData
FDeferredLightData ConvertToDeferredLight(
	FDirectionalLightData In, 
	float InSpecularScale, 
	inout half4 OutPreviewShadowMapChannelMask,
	inout uint OutLightingChannelMask)
{
	FDeferredLightData Out	= (FDeferredLightData)0;
	Out.Color				= In.DirectionalLightColor;
	Out.FalloffExponent		= 0;
	Out.Direction			= In.DirectionalLightDirection;
	Out.DistanceFadeMAD		= In.DirectionalLightDistanceFadeMAD;
	Out.bRadialLight		= false;
	Out.SpecularScale		= In.DirectionalLightSpecularScale * InSpecularScale;
	Out.DiffuseScale		= In.DirectionalLightDiffuseScale;
	Out.ShadowedBits		= (In.DirectionalLightSceneInfoExtraDataPacked & 0xFF) != 0 ? 1 : 0;
	Out.HairTransmittance	= InitHairTransmittanceData();
	Out.SourceRadius		= In.DirectionalLightSourceRadius;
	Out.SoftSourceRadius	= In.DirectionalLightSoftSourceRadius;
	Out.IESAtlasIndex		= -1;
	Out.LightFunctionAtlasLightIndex = In.LightFunctionAtlasLightIndex;
	Out.bAffectsTranslucentLighting = In.bAffectsTranslucentLighting;

	// Shadow
	// * Static shadowing uses ShadowMapChannel, 
	// * Dynamic shadows are packed into light attenuation using PreviewShadowMapChannel
	// Data Layout on [10:0] with:
	// * ShadowMapChannelMask	= [3:0]
	// * PreviewShadowMapChannel= [7:4] 
	// * LightincChannelMask    = [8:10] 
	Out.ShadowMapChannelMask		= UnpackShadowMapChannelMask(In.DirectionalLightSceneInfoExtraDataPacked);
	OutPreviewShadowMapChannelMask	= UnpackShadowMapChannelMask(In.DirectionalLightSceneInfoExtraDataPacked >> 4);
	OutLightingChannelMask			= UnpackLightingChannelMask(In.DirectionalLightSceneInfoExtraDataPacked);

	return Out;
}

// Helper function for converting FLightSceneData into FDeferredLightData
FDeferredLightData ConvertToDeferredLight(FLightSceneData In, FLightViewData InViewData)
{
	FDeferredLightData Out = (FDeferredLightData)0;

	const uint LightType = UnpackLightType(InViewData.LightSceneInfoExtraDataPacked);

	Out.TranslatedWorldPosition = InViewData.TranslatedWorldPosition;
	Out.InvRadius				= In.InvRadius;
	Out.Color					= InViewData.Color;
	Out.FalloffExponent			= In.FalloffExponent;
	Out.Direction				= In.Direction;
	Out.SpotAngles				= In.SpotAngles;
	Out.SourceRadius			= In.SourceRadius;
	Out.SourceLength			= In.SourceLength;
	Out.Tangent					= In.Tangent;
	Out.SoftSourceRadius		= In.SoftSourceRadius;
	Out.bInverseSquared			= In.FalloffExponent == 0 && (LightType != LIGHT_TYPE_DIRECTIONAL); // Directional lights don't use 'inverse squared attenuation'
	Out.SpecularScale			= In.SpecularScale;
	Out.DiffuseScale			= In.DiffuseScale;
	Out.bRadialLight			= LightType != LIGHT_TYPE_DIRECTIONAL;
	Out.bSpotLight				= LightType == LIGHT_TYPE_SPOT;
	Out.bRectLight				= LightType == LIGHT_TYPE_RECT;
	Out.HairTransmittance		= InitHairTransmittanceData();

	Out.RectLightData.BarnCosAngle = In.RectLightBarnCosAngle;
	Out.RectLightData.BarnLength = In.RectLightBarnLength;
	Out.RectLightData.AtlasData.AtlasMaxLevel = InViewData.RectLightAtlasMaxLevel;
	Out.RectLightData.AtlasData.AtlasUVOffset = InViewData.RectLightAtlasUVOffset;
	Out.RectLightData.AtlasData.AtlasUVScale = InViewData.RectLightAtlasUVScale;

	Out.IESAtlasIndex					= InViewData.IESAtlasIndex;
	Out.LightFunctionAtlasLightIndex	= UnpackLightFunctionAtlasIndex(InViewData.LightSceneInfoExtraDataPacked);

	Out.bAffectsTranslucentLighting = UnpackAffectsTranslucentLighting(InViewData.LightSceneInfoExtraDataPacked);

	// Shadow
	// * Static shadowing uses ShadowMapChannel, 
	// * Dynamic shadows are packed into light attenuation using PreviewShadowMapChannel
	// Data Layout on [10:0] with:
	// * ShadowMapChannelMask	= [3:0]
	// * PreviewShadowMapChannel= [7:4] 
	// * LightingChannelMask    = [8:10] 
	Out.ShadowedBits				= (InViewData.LightSceneInfoExtraDataPacked & 0xFF) != 0 ? 1 : 0;
	Out.ShadowMapChannelMask		= UnpackShadowMapChannelMask(InViewData.LightSceneInfoExtraDataPacked);
	return Out;
}

// Helper function for converting FLightSceneData into FLightShaderParameters
FLightShaderParameters ConvertToLightShaderParameters(FLightSceneData In, FLightViewData InViewData)
{
	FLightShaderParameters Out;
	Out.TranslatedWorldPosition	= DFFastAddDemote(In.WorldPosition, PrimaryView.PreViewTranslation);
	Out.InvRadius				= In.InvRadius;
	Out.Color					= InViewData.Color;
	Out.FalloffExponent			= In.FalloffExponent;
	Out.Direction				= In.Direction;
	Out.Tangent					= In.Tangent;
	Out.SpotAngles				= In.SpotAngles;
	Out.SpecularScale			= 1.0f; // In.SpecularScale;
	Out.DiffuseScale			= In.DiffuseScale;
	Out.SourceRadius			= In.SourceRadius;
	Out.SoftSourceRadius		= In.SoftSourceRadius;
	Out.SourceLength			= In.SourceLength;
	Out.RectLightBarnCosAngle	= In.RectLightBarnCosAngle;
	Out.RectLightBarnLength		= In.RectLightBarnLength;
	Out.RectLightAtlasUVOffset	= InViewData.RectLightAtlasUVOffset;
	Out.RectLightAtlasUVScale	= InViewData.RectLightAtlasUVScale;
	Out.RectLightAtlasMaxLevel	= InViewData.RectLightAtlasMaxLevel;
	return Out;
}

// Helper function for converting FLocalLightData into FSimpleDeferredLightData
FSimpleDeferredLightData ConvertToSimpleLight(FLocalLightData In)
{
	FSimpleDeferredLightData Out = (FSimpleDeferredLightData)0;
	Out.TranslatedWorldPosition = In.Internal.LightPositionAndInvRadius.xyz;
	Out.InvRadius				= In.Internal.LightPositionAndInvRadius.w;
	Out.Color					= UnpackLightColor(In);
	Out.FalloffExponent			= UnpackLightFalloffExponent(In);
	Out.bInverseSquared			= Out.FalloffExponent == 0;
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions

// Return true if the point is influenced by the light
bool IsLightVisible(FForwardLightData InLight, float3 TranslatedWorldPosition)
{
	float InvLightRadiusSq = InLight.LightPositionAndInvRadius.w * InLight.LightPositionAndInvRadius.w;
	return length2(TranslatedWorldPosition - InLight.LightPositionAndInvRadius.xyz) * InvLightRadiusSq <= 1.0f;
}

bool IsLightVisible(FLocalLightData InLight, float3 TranslatedWorldPosition)
{
	return IsLightVisible(InLight.Internal, TranslatedWorldPosition);
}