// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightData.ush"

// Build the light data struct using the DeferredLightUniforms and light defines
// We are heavily relying on the shader compiler to optimize out constant subexpressions in GetDynamicLighting()
FDeferredLightData InitDeferredLightFromUniforms(uint InLightType)
{
	const bool bIsRadial = InLightType != LIGHT_TYPE_DIRECTIONAL;

	FDeferredLightData Out;
	Out.TranslatedWorldPosition = GetDeferredLightTranslatedWorldPosition();
	Out.InvRadius				= DeferredLightUniforms.InvRadius;
	Out.Color					= DeferredLightUniforms.Color;
	Out.FalloffExponent			= DeferredLightUniforms.FalloffExponent;
	Out.Direction				= DeferredLightUniforms.Direction;
	Out.Tangent					= DeferredLightUniforms.Tangent;
	Out.SpotAngles				= DeferredLightUniforms.SpotAngles;
	Out.SourceRadius			= DeferredLightUniforms.SourceRadius;
	Out.SourceLength			= bIsRadial ? DeferredLightUniforms.SourceLength : 0;
	Out.SoftSourceRadius		= DeferredLightUniforms.SoftSourceRadius;
	Out.SpecularScale			= DeferredLightUniforms.SpecularScale;
	Out.DiffuseScale			= DeferredLightUniforms.DiffuseScale;
	Out.ContactShadowLength		= abs(DeferredLightUniforms.ContactShadowLength);
	Out.ContactShadowLengthInWS = DeferredLightUniforms.ContactShadowLength < 0.0f;
	Out.ContactShadowCastingIntensity = DeferredLightUniforms.ContactShadowCastingIntensity;
	Out.ContactShadowNonCastingIntensity = DeferredLightUniforms.ContactShadowNonCastingIntensity;
	Out.DistanceFadeMAD			= DeferredLightUniforms.DistanceFadeMAD;
	Out.ShadowMapChannelMask	= DeferredLightUniforms.ShadowMapChannelMask;
	Out.ShadowedBits			= DeferredLightUniforms.ShadowedBits;
	Out.bInverseSquared			= bIsRadial && DeferredLightUniforms.FalloffExponent == 0; // Directional lights don't use 'inverse squared attenuation'
	Out.bRadialLight			= bIsRadial;
	Out.bSpotLight				= InLightType == LIGHT_TYPE_SPOT;
	Out.bRectLight				= InLightType == LIGHT_TYPE_RECT;

	Out.RectLightData.BarnCosAngle				= DeferredLightUniforms.RectLightBarnCosAngle;
	Out.RectLightData.BarnLength				= DeferredLightUniforms.RectLightBarnLength;
	Out.RectLightData.AtlasData.AtlasMaxLevel	= DeferredLightUniforms.RectLightAtlasMaxLevel;
	Out.RectLightData.AtlasData.AtlasUVOffset	= DeferredLightUniforms.RectLightAtlasUVOffset;
	Out.RectLightData.AtlasData.AtlasUVScale	= DeferredLightUniforms.RectLightAtlasUVScale;

	Out.IESAtlasIndex = DeferredLightUniforms.IESAtlasIndex;
	Out.LightFunctionAtlasLightIndex = DeferredLightUniforms.LightFunctionAtlasLightIndex;

	Out.bAffectsTranslucentLighting = DeferredLightUniforms.bAffectsTranslucentLighting;

	Out.HairTransmittance		= InitHairTransmittanceData();
	return Out;
}

FDeferredLightData InitDeferredLightFromUniforms()
{
	uint LightType = LIGHT_TYPE_POINT;
	if (DeferredLightUniforms.SpotAngles.x > -2.0f)			{ LightType = LIGHT_TYPE_SPOT; } 
	if (DeferredLightUniforms.RectLightBarnLength > -2.0f)	{ LightType = LIGHT_TYPE_RECT; }
	return InitDeferredLightFromUniforms(LightType);
}

// Special version for mobil. Used only for punctual local light (point or spot)
FDeferredLightData SetupLightDataForStandardDeferred_Mobile(uint InLightType)
{
	FDeferredLightData Out = (FDeferredLightData)0;
	Out.TranslatedWorldPosition = GetDeferredLightTranslatedWorldPosition();
	Out.InvRadius				= DeferredLightUniforms.InvRadius;
	Out.Color					= DeferredLightUniforms.Color;
	Out.FalloffExponent			= DeferredLightUniforms.FalloffExponent;
	Out.Direction				= DeferredLightUniforms.Direction;
	Out.SpotAngles				= DeferredLightUniforms.SpotAngles;
	Out.SpecularScale			= DeferredLightUniforms.SpecularScale;
	Out.DiffuseScale			= DeferredLightUniforms.DiffuseScale;
	Out.ShadowMapChannelMask	= DeferredLightUniforms.ShadowMapChannelMask;
	Out.ShadowedBits			= DeferredLightUniforms.ShadowedBits;
	Out.bInverseSquared			= DeferredLightUniforms.FalloffExponent == 0; // Directional lights don't use 'inverse squared attenuation'
	Out.bRadialLight			= true;
	Out.bSpotLight				= InLightType == LIGHT_TYPE_SPOT;
	Out.bRectLight				= false;
	Out.HairTransmittance		= InitHairTransmittanceData();

	return Out;
}

bool IsDeferredSpotlight()
{
	return DeferredLightUniforms.SpotAngles.x > -2.0f;
}