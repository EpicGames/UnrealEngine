// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightFunctionAtlasCommon.usf
=============================================================================*/

#pragma once

#include "../Common.ush"
#include "/Engine/Shared/LightDefinitions.h"

// In order to use function from that file, one must use FLightFunctionAtlasGlobalParameters

// Trick so that we can sample from LightFunctionAtlas when it is a global UB or when it is a Struct part of another UB.
#ifdef LightFunctionAtlasStruct
#undef  LightFunctionAtlas
#define LightFunctionAtlas LightFunctionAtlasStruct
#endif

#if COLORED_LIGHT_FUNCTION_ATLAS
#define FLightFunctionColor float3
#else
#define FLightFunctionColor float
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define LightFunctionAtlasSampler	View.SharedBilinearClampedSampler
#else
#define LightFunctionAtlasSampler	LightFunctionAtlas.LightFunctionAtlasSampler
#endif

struct FLightFunctionSlotData
{
	uint LightType;
	float DisabledBrightness;
	float FadeDistance;
	float TanOuterAngle;
	float2 MinUV;
	float4x4 TranslatedWorldToLight;
};

FLightFunctionSlotData GetLightFunctionSlotData(uint LightFunctionAtlasLightIndex)
{
	FLightFunctionSlotData Out;

	const uint BaseBufferIndex = LightFunctionAtlasLightIndex * 5; // See FAtlasLightInfoData

	const float4 LightFunctionLightDataParameters = LightFunctionAtlas.LightInfoDataBuffer[BaseBufferIndex + 0];
	const uint PackedLightInfoDataParams0 = asuint(LightFunctionLightDataParameters.y);
	const uint PackedLightInfoDataParams1 = asuint(LightFunctionLightDataParameters.z);

	Out.LightType = PackedLightInfoDataParams0 & 0xFF;
	Out.DisabledBrightness = f16tof32((PackedLightInfoDataParams0 >> 8) & 0xFFFF);
	Out.FadeDistance = LightFunctionLightDataParameters.x;
	Out.TanOuterAngle = LightFunctionLightDataParameters.w;
	Out.MinUV = float2(((PackedLightInfoDataParams1) & 0xFFFF) * rcp(65536.0f), ((PackedLightInfoDataParams1 >> 16) & 0xFFFF) * rcp(65536.0f));
	Out.TranslatedWorldToLight = float4x4(
		LightFunctionAtlas.LightInfoDataBuffer[BaseBufferIndex + 1],
		LightFunctionAtlas.LightInfoDataBuffer[BaseBufferIndex + 2],
		LightFunctionAtlas.LightInfoDataBuffer[BaseBufferIndex + 3],
		LightFunctionAtlas.LightInfoDataBuffer[BaseBufferIndex + 4]);

	return Out;
}

FLightFunctionColor SampleLightFunctionSlot(FLightFunctionSlotData Slot, float2 LightFunctionUV)
{
	// Slot min uv and size already account for half texel offsets
	float2 AtlasUV = Slot.MinUV + saturate(LightFunctionUV) * LightFunctionAtlas.Slot_UVSize.xx;

#if COLORED_LIGHT_FUNCTION_ATLAS
	FLightFunctionColor LightFunctionColor = Texture2DSampleLevel(LightFunctionAtlas.LightFunctionAtlasTexture, LightFunctionAtlasSampler, AtlasUV, 0).rgb;
#else
	FLightFunctionColor LightFunctionColor = Texture2DSampleLevel(LightFunctionAtlas.LightFunctionAtlasTexture, LightFunctionAtlasSampler, AtlasUV, 0).x;
#endif

	return LightFunctionColor * LightFunctionColor; // Simple Gamma decompression
}

FLightFunctionColor ApplyLightFunctionFade(float3 TranslatedWorldPosition, FLightFunctionSlotData Slot, FLightFunctionColor LightFunctionColor)
{
	// Calculate radial view distance for stable fading
	float ViewDistance = GetDistanceToCameraFromViewVector(PrimaryView.TranslatedWorldCameraOrigin - TranslatedWorldPosition);
	float DistanceFadeAlpha = saturate((Slot.FadeDistance - ViewDistance) / (Slot.FadeDistance * .2f));
	// Fade to disabled based on FadeDistance
	LightFunctionColor = lerp(Slot.DisabledBrightness, LightFunctionColor, DistanceFadeAlpha);
	// Fade to disabled based on ShadowFadeFraction
	const float ShadowFadeFraction = 1.0f; // Not setup
	LightFunctionColor = LightFunctionColor; // lerp(DisabledBrightness, LightFunctionColor, ShadowFadeFraction);

	return LightFunctionColor;
}

FLightFunctionColor GetLocalLightFunctionCommon(float3 TranslatedWorldPosition, uint LightFunctionAtlasLightIndex)
{
	if (LightFunctionAtlasLightIndex == 0)
	{
		// No valid light function assigned to the light.
		return 1.0f;
	}

	FLightFunctionSlotData Slot = GetLightFunctionSlotData(LightFunctionAtlasLightIndex);

	float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPosition, 1), Slot.TranslatedWorldToLight);
	float3 LightVector = HomogeneousShadowPosition.zyx / HomogeneousShadowPosition.w;
	float2 LightFunctionUV = LightVector.xy;

	switch (Slot.LightType)
	{

	case LIGHT_TYPE_SPOT:
	{
		LightFunctionUV = LightVector.xy / (LightVector.z * Slot.TanOuterAngle) * .5f + .5f;
		break;
	}

	case LIGHT_TYPE_POINT:
	{
		float3 UnitLightVector = normalize(LightVector);
		// Setup 2d UVs for a point light that map the texture using spherical coordinates, which is how max handles a 2d texture projector on a point light
		// Artists should use a cubemap indexed by a light vector to get better quality
		LightFunctionUV = float2((atan2(UnitLightVector.y, UnitLightVector.x) + PI) / (2 * PI), acos(UnitLightVector.z) / PI);
		break;
	}

	case LIGHT_TYPE_RECT:
	{
		if (Slot.TanOuterAngle > 0.0f)
		{
			LightFunctionUV = LightVector.xy / (LightVector.z * Slot.TanOuterAngle) * .5f + .5f;
		}
		break;
	}

	//case LIGHT_TYPE_DIRECTIONAL:
	default:
	{
		// no op
		break;
	}

	}

	LightFunctionUV = frac(LightFunctionUV);

	FLightFunctionColor LightFunctionColor = SampleLightFunctionSlot(Slot, LightFunctionUV);
	LightFunctionColor = ApplyLightFunctionFade(TranslatedWorldPosition, Slot, LightFunctionColor);

	return LightFunctionColor;
}