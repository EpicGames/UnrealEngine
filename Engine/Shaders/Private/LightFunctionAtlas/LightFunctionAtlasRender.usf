// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderLightFunctionAtlas.usf
=============================================================================*/

// This define is for the light function common evaluation to only be based on UV stored in the light vector only.
#define LIGHT_FUNCTION_ATLAS 1

// Used to trigger shader recompiles, guid should be regenerated on merge conflicts
#pragma message("UESHADERMETADATA_VERSION D13B08DE-2E66-447C-AF33-27D896D280CA")

#include "../Common.ush"
#include "/Engine/Generated/Material.ush"
#include "../LightFunctionCommon.ush"

float2 LightFunctionTexelSize;
float4 SvPositionToUVScaleBias;

void Main(
	in noperspective float4 UV : TEXCOORD0,
	in FStereoPSInput StereoInput,
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0
)
{
	float4 TranslatedWorldPosition = float4(0, 0, 0, 1);

	float Depth = 0.000001f;
	float2 ScreenPosition = 0.0f;

	float3 LightVector = 0.0f;
	float3 LightFunction = 0.0f;

//	ScreenPosition = (SvPosition.xy * LightFunctionTexelSize - float2(0.5f, 0.5f)) * float2(2, -2);
	float2 UVs = (SvPosition.xy - SvPositionToUVScaleBias.zw) * SvPositionToUVScaleBias.xy;

#if 1

	// Generate a local space light vector that is at least correct for point lights
	float Theta = (UVs.x * (2 * PI)) - PI;
	float Phi = UVs.y * PI;
	float SinTheta, CosTheta;
	float SinPhi, CosPhi;
	sincos(Theta, SinTheta, CosTheta);
	sincos(Phi, SinPhi, CosPhi);
	float X = CosTheta * SinPhi;
	float Y = SinTheta * SinPhi;
	float Z = CosPhi;
	LightVector = normalize(float3(X, Y, Z)); 

	LightFunction = GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz, UVs.xy);

#else

	// TODO Super sample 4x
	const float2 UVOffset = 0.25f * LightFunctionTexelSize;

	LightVector.xy = UVs.xy + UVOffset * float2(-1.0f, -1.0f);
	LightFunction += GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz);

	LightVector.xy = UVs.xy + UVOffset * float2(-1.0f, 1.0f);
	LightFunction += GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz);

	LightVector.xy = UVs.xy + UVOffset * float2(1.0f, 1.0f);
	LightFunction += GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz);

	LightVector.xy = UVs.xy + UVOffset * float2(1.0f, -1.0f);
	LightFunction += GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz);

	LightFunction *= 0.25f;

#endif

#if COLORED_LIGHT_FUNCTION_ATLAS

	LightFunction = sqrt(max(0.0f, LightFunction));	// Simple Gamma compression
	OutColor = float4(LightFunction, 0.0f);

#else

	float GreyScale = dot(LightFunction, .3333f).x;
	GreyScale = sqrt(max(0.0f, GreyScale));						// Simple Gamma compression
	OutColor = float4(GreyScale, 0.0f, 0.0f, 0.0f);

#endif
}
