// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightFunctionCommon.usf: Utility functions for light functions.
=============================================================================*/

#include "/Engine/Private/Substrate/Substrate.ush" 


#ifndef LIGHT_FUNCTION_ATLAS
#define LIGHT_FUNCTION_ATLAS 0
#endif

// This ifdef is to allow shaders(notably the ratracing miss shader implementing light functions) to override where these parameters come from.
#ifndef LightFunctionParameters
/** Tan of spotlight cone outer angle in x, ShadowFadeFraction in y, IsSpotLight in z, IsPointLight in w. */
float4 LightFunctionParameters;
float4x4 LightFunctionTranslatedWorldToLight;
#endif

/** 
 * Calculates the light function color with the given light vector. 
 * LightVector is the vector from the light to the position being shaded, in the light's coordinate space.
 */
float3 GetLightFunctionColor(float3 LightVector, float3 TranslatedWorldSpace, in out FMaterialPixelParameters MaterialParameters
#if LIGHT_FUNCTION_ATLAS
	, float2 UVs
#endif
)
{

#if LIGHT_FUNCTION_ATLAS

	// This path is used when rendering a light function into a 2D Atlas
	float2 LightFunctionUVs = UVs;

#else // LIGHT_FUNCTION_ATLAS

	// Swizzle so that LightVector.xy are perpendicular to the light direction and LightVector.z is distance along the light direction
	LightVector.xyz = LightVector.zyx;

	// By default, map textures using the vectors perpendicular to the light direction
	float2 LightFunctionUVs = LightVector.xy;

	if (LightFunctionParameters.z > 0)
	{
		// For spotlights, setup UVs that go from 1 at one side of the spotlight cone to 0 at the other side, at any distance from the light
		// This minimizes artist setup when they just want to use the spotlight like a projector
		LightFunctionUVs = LightVector.xy / (LightVector.z * LightFunctionParameters.x) * .5f + .5f;
	}
	else if (LightFunctionParameters.w > 0)
	{
		float3 UnitLightVector = normalize(LightVector);
		// Setup 2d UVs for a pointlight that map the texture using spherical coordinates, which is how max handles a 2d texture projector on a point light
		// Artists should use a cubemap indexed by a light vector to get better quality
		LightFunctionUVs = float2((atan2(UnitLightVector.y, UnitLightVector.x) + PI) / (2 * PI), acos(UnitLightVector.z) / PI);
	}

#endif // LIGHT_FUNCTION_ATLAS

#if NUM_MATERIAL_TEXCOORDS
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_MATERIAL_TEXCOORDS;CoordinateIndex++)
	{
		MaterialParameters.TexCoords[CoordinateIndex] = LightFunctionUVs;
	}
#endif
	MaterialParameters.VertexColor = 1;
	MaterialParameters.CameraVector = LightVector.xyz;
	MaterialParameters.ReflectionVector = LightVector.xyz;
	MaterialParameters.LightVector = LightVector.xyz;
	MaterialParameters.AbsoluteWorldPosition = DFFastSubtract(TranslatedWorldSpace, PrimaryView.PreViewTranslation);
	MaterialParameters.WorldPosition_CamRelative = MaterialParameters.WorldPosition_NoOffsets_CamRelative = TranslatedWorldSpace;
	MaterialParameters.ScreenPosition = mul(float4(TranslatedWorldSpace, 1.0f), PrimaryView.TranslatedWorldToClip);
	MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	// Light function node forces a single unlit BSDF. We get the raw data from the Substrate light function node from the raw BSDF on the tree.
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	FSubstrateBSDF UnlitBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];
	UnlitBSDF.SubstrateSanitizeBSDF(); // required wehen not calling SubstrateUpdateTreeUnlit

	return SubstrateGetBSDFEmissive(UnlitBSDF);

#else // SUBSTRATE_ENABLED

   #if COMPUTESHADER
	return GetMaterialEmissiveForCS(MaterialParameters);
   #else
	return GetMaterialEmissive(PixelMaterialInputs);
   #endif

#endif // SUBSTRATE_ENABLED
}

float3 GetLightFunctionColor(float3 LightVector, float3 TranslatedWorldSpace
#if LIGHT_FUNCTION_ATLAS
	, float2 UVs
#endif
)
{
	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	return GetLightFunctionColor(LightVector, TranslatedWorldSpace, MaterialParameters
#if LIGHT_FUNCTION_ATLAS
	, UVs
#endif
	);
}
