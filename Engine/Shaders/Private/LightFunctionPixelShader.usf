// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightFunction.usf: Pixel shader for computing a light function.
=============================================================================*/

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "LightFunctionCommon.ush"
#include "DynamicLightingCommon.ush"

// Used to trigger shader recompiles, guid should be regenerated on merge conflicts
#pragma message("UESHADERMETADATA_VERSION 02A826D0-F3F6-4726-A187-762FFB6D17CA")

// used like this: mul(float4(SvPosition.xyz, 1), SvPositionToLight);
float4x4 SvPositionToLight;

/** Fade distance in x, disabled brightness in y, output for preview shadows mask in z, hair strands in w */
float4 LightFunctionParameters2;

Texture2D HairOnlyDepthTexture;

void Main(
	// in float4 ScreenPosition : TEXCOORD0, // Not used and not output in VS
	in float4 SvPosition : SV_Position,		// after all interpolators
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();

	float2 ScreenUV = SvPositionToBufferUV(SvPosition);

	// make SvPosition appear to be rasterized with the depth from the depth buffer
	SvPosition.z = LookupDeviceZ(ScreenUV);
#if HAIR_STRANDS_SUPPORTED
	BRANCH
	if (LightFunctionParameters2.w > 0)
	{
		SvPosition.z = Texture2DSampleLevel(HairOnlyDepthTexture, SceneTexturesStruct_SceneDepthTextureSampler, ScreenUV, 0).r; 
	}
#endif

	float3 LightVector;
	{
		float4 Hom = mul(float4(SvPosition.xyz, 1), SvPositionToLight);
		LightVector = Hom.xyz / Hom.w;
	}
	
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);

	// Calculate radial view distance for stable fading
	float ViewDistance = GetDistanceToCameraFromViewVector(PrimaryView.TranslatedWorldCameraOrigin - TranslatedWorldPosition);

	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	InitializeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback);
#endif

	float GreyScale;
	{
		float3 Color = GetLightFunctionColor(LightVector, TranslatedWorldPosition, MaterialParameters);
		GreyScale = dot(Color, .3333f);
	}

	float DistanceFadeAlpha = saturate((LightFunctionParameters2.x - ViewDistance) / (LightFunctionParameters2.x * .2f));
	// Fade to disabled based on LightFunctionFadeDistance
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, DistanceFadeAlpha);

	// Fade to disabled based on ShadowFadeFraction
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, LightFunctionParameters.y);

#if FORWARD_SHADING
	float LightInfluenceMask = GetLightInfluenceMask(TranslatedWorldPosition);
	GreyScale = lerp(1, GreyScale, LightInfluenceMask);
	OutColor = EncodeLightAttenuation(GreyScale);
#else
	float EncodedLightAttenuation = EncodeLightAttenuation(GreyScale);

	// Light function shadows write to the blue channel.
	OutColor = lerp(float4(1, 1, EncodedLightAttenuation.xx), EncodedLightAttenuation.xxxx, LightFunctionParameters2.z);
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback, SvPosition, View.VTFeedbackBuffer);
#endif
}
