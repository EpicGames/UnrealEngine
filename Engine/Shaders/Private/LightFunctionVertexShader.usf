// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightFunction.usf: Vertex shader for computing a light function.
=============================================================================*/

#include "Common.ush"

float4 StencilingGeometryPosAndScale;

void Main(
	in float4 InPosition : ATTRIBUTE0,
	out float4 OutPosition : SV_POSITION
	)
{
	float3 WorldPosition = InPosition.xyz * StencilingGeometryPosAndScale.w + StencilingGeometryPosAndScale.xyz;
	OutPosition = mul(float4(WorldPosition,1), View.TranslatedWorldToClip);
}

