// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightGridCommon.ush
=============================================================================*/

#pragma once 

#include "LightData.ush"

// If this is changed, the LightGridUses16BitBuffers function from LightGridInjection.cpp should also be updated.
#define LIGHT_GRID_USES_16BIT_BUFFERS (PLATFORM_SUPPORTS_BUFFER_LOAD_TYPE_CONVERSION && !SHADING_PATH_MOBILE)

// ForwardLightStruct.CulledLightDataGrid could be 32bit StructuredBuffer or 16bit Buffer, make sure to access it from this function
uint GetCulledLightDataGrid(uint GridIndex)
{
#if LIGHT_GRID_USES_16BIT_BUFFERS
	return ForwardLightStruct.CulledLightDataGrid16Bit[GridIndex];
#else
	return ForwardLightStruct.CulledLightDataGrid32Bit[GridIndex];
#endif
}

struct FLightGridData
{
	uint LightGridPixelSizeShift;
	float3 LightGridZParams;
	int3 CulledGridSize;
};

FLightGridData GetLightGridData()
{
	FLightGridData Result;
	Result.LightGridPixelSizeShift = ForwardLightStruct.LightGridPixelSizeShift;
	Result.LightGridZParams = ForwardLightStruct.LightGridZParams;
	Result.CulledGridSize = ForwardLightStruct.CulledGridSize;
	return Result;
}

uint3 ComputeLightGridCellCoordinate(uint2 PixelPos, float SceneDepth)
{
	const FLightGridData GridData = GetLightGridData();
	uint ZSlice = (uint)(max(0, log2(SceneDepth * GridData.LightGridZParams.x + GridData.LightGridZParams.y) * GridData.LightGridZParams.z));
	ZSlice = min(ZSlice, (uint)(GridData.CulledGridSize.z - 1));
	return uint3(PixelPos >> GridData.LightGridPixelSizeShift, ZSlice);
}

uint ComputeLightGridCellIndex(uint3 GridCoordinate, uint EyeIndex)
{
	const FLightGridData GridData = GetLightGridData();
	uint Index = (GridCoordinate.z * GridData.CulledGridSize.y + GridCoordinate.y) * GridData.CulledGridSize.x + GridCoordinate.x;
#if (INSTANCED_STEREO || MOBILE_MULTI_VIEW)
	Index += (EyeIndex == 0) ? 0 : ForwardLightStruct.CulledBufferOffsetISR;
#endif
	return Index;
}

uint ComputeLightGridCellIndex(uint2 PixelPos, float SceneDepth, uint EyeIndex)
{
	return ComputeLightGridCellIndex(ComputeLightGridCellCoordinate(PixelPos, SceneDepth), EyeIndex);
}

#ifndef NUM_CULLED_LIGHTS_GRID_STRIDE
#define NUM_CULLED_LIGHTS_GRID_STRIDE 0
#endif

#ifndef FORWARD_LIGHT_DATA_STRIDE
#define FORWARD_LIGHT_DATA_STRIDE 0
#endif

uint GetNumLocalLights()
{
	return ForwardLightStruct.NumLocalLights;
}

uint GetMaxLightsPerCell()
{
	return ForwardLightStruct.MaxCulledLightsPerCell;
}

uint PackCulledLightsGridHeader0(uint NumVisibleLights, uint NumVisibleMegaLights)
{
	return (NumVisibleMegaLights << 16) | (NumVisibleLights & 0xFFFF);
}

uint PackCulledLightsGridHeader1(uint CulledLightDataStart, bool bHasRectLight, bool bHasTexturedLight)
{
	return (bHasTexturedLight ? 0x80000000 : 0) | (bHasRectLight ? 0x40000000 : 0) | (CulledLightDataStart & 0x3FFFFFFF);
}

void UnpackCulledLightsGridHeader0(uint PackedData0, out uint NumVisibleLights, out uint NumVisibleMegaLights)
{
	NumVisibleLights = PackedData0 & 0xFFFF;
	NumVisibleMegaLights = (PackedData0 >> 16) & 0xFFFF;
}

void UnpackCulledLightsGridHeader1(uint PackedData1, out uint CulledLightDataStart, out bool bHasRectLight, out bool bHasTexturedLight)
{
	CulledLightDataStart = (PackedData1 & 0x3FFFFFFF);
	bHasRectLight = (PackedData1 & 0x40000000) != 0;
	bHasTexturedLight = (PackedData1 & 0x80000000) != 0;
}

struct FCulledLightsGridHeader
{
	uint NumLights;
	uint NumMegaLights;
	uint DataStartIndex;
	uint MegaLightsDataStartIndex;
	bool bHasRectLight;
	bool bHasTexturedLight;
};

FCulledLightsGridHeader GetCulledLightsGridHeader(uint GridIndex)
{
	FCulledLightsGridHeader Result;

	const uint PackedData0 = ForwardLightStruct.NumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 0];
	UnpackCulledLightsGridHeader0(PackedData0, Result.NumLights, Result.NumMegaLights);
	Result.NumLights = min(Result.NumLights, ForwardLightStruct.NumLocalLights);
	Result.NumMegaLights = min(Result.NumMegaLights, ForwardLightStruct.NumLocalLights);

	const uint PackedData1 = ForwardLightStruct.NumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 1];
	UnpackCulledLightsGridHeader1(PackedData1, Result.DataStartIndex, Result.bHasRectLight, Result.bHasTexturedLight);

	Result.MegaLightsDataStartIndex = Result.DataStartIndex + Result.NumLights - Result.NumMegaLights;

	return Result;
}

struct FCulledReflectionCapturesGridHeader
{
	uint NumReflectionCaptures;
	uint DataStartIndex;
};

FCulledReflectionCapturesGridHeader GetCulledReflectionCapturesGridHeader(uint GridIndex)
{
	FCulledReflectionCapturesGridHeader Result;
	const uint NumCulledEntryIndex = (ForwardLightStruct.NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE;
	Result.NumReflectionCaptures = min(ForwardLightStruct.NumCulledLightsGrid[NumCulledEntryIndex + 0], ForwardLightStruct.NumReflectionCaptures);
	Result.DataStartIndex = ForwardLightStruct.NumCulledLightsGrid[NumCulledEntryIndex + 1];
	return Result;
}

FDirectionalLightData GetDirectionalLightData()
{
	FDirectionalLightData Result;
	Result.HasDirectionalLight						= ForwardLightStruct.HasDirectionalLight;
	Result.DirectionalLightSceneInfoExtraDataPacked = ForwardLightStruct.DirectionalLightSceneInfoExtraDataPacked;
	Result.DirectionalLightDistanceFadeMAD			= ForwardLightStruct.DirectionalLightDistanceFadeMAD;
	Result.DirectionalLightColor					= ForwardLightStruct.DirectionalLightColor;
	Result.DirectionalLightDirection				= ForwardLightStruct.DirectionalLightDirection;
	Result.DirectionalLightSourceRadius				= ForwardLightStruct.DirectionalLightSourceRadius;
	Result.DirectionalLightSoftSourceRadius			= ForwardLightStruct.DirectionalLightSoftSourceRadius;
	Result.DirectionalLightSpecularScale			= ForwardLightStruct.DirectionalLightSpecularScale;
	Result.DirectionalLightDiffuseScale				= ForwardLightStruct.DirectionalLightDiffuseScale;
	Result.LightFunctionAtlasLightIndex				= ForwardLightStruct.DirectionalLightLightFunctionAtlasLightIndex;
	Result.bAffectsTranslucentLighting				= ForwardLightStruct.DirectionalLightAffectsTranslucentLighting;
	Result.bHandledByMegaLights                     = ForwardLightStruct.DirectionalLightHandledByMegaLights;
	return Result;
}

FForwardLightData GetForwardLightData_Internal(
	uint LightIndex, uint EyeIndex,
	float4 InData0, float4 InData1, float4 InData2, float4 InData3, float4 InData4, float4 InData5)
{
	FForwardLightData Out = (FForwardLightData)0;

	Out.LightPositionAndInvRadius                      = InData0;
	Out.LightColorAndIdAndFalloffExponentAndRayEndBias = InData1;
	Out.LightDirectionAndSceneInfoExtraDataPacked      = InData2;
	Out.SpotAnglesAndSourceRadiusPacked                = InData3;
	Out.LightTangentAndIESDataAndSpecularScale         = InData4;
	Out.RectData                                       = InData5.xyz;
	// See PackVirtualShadowMapIdAndPrevLocalLightIndex
	Out.VirtualShadowMapId                             = int(asuint(InData5.w) >> 16U) - 1;
	Out.PrevLocalLightIndex                            = int(asuint(InData5.w) & 0xFFFF) - 1;
	Out.LightSceneId                                   = int(Out.LightColorAndIdAndFalloffExponentAndRayEndBias.z);

#if (INSTANCED_STEREO || MOBILE_MULTI_VIEW)
	if (EyeIndex != 0)
	{
		Out.LightPositionAndInvRadius.xyz += ForwardLightStruct.PreViewTranslationOffsetISR.xyz;
	}
#endif

	return Out;
}

FForwardLightData GetForwardLightData(uint LightIndex, uint EyeIndex)
{
	const uint LightBaseIndex = LightIndex * FORWARD_LIGHT_DATA_STRIDE;

	return GetForwardLightData_Internal(
		LightIndex, EyeIndex,
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 0],
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 1],
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 2],
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 3],
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 4],
		ForwardLightStruct.ForwardLightBuffer[LightBaseIndex + 5]);
}

FLocalLightData GetLocalLightData(uint LightIndex, uint EyeIndex)
{
	FLocalLightData Out = (FLocalLightData)0;
	Out.Internal = GetForwardLightData(LightIndex, EyeIndex);
	return Out;
}

FForwardLightData GetForwardLightDataFromGrid(uint GridIndex, uint EyeIndex)
{
	uint LocalLightIndex = GetCulledLightDataGrid(GridIndex);
	return GetForwardLightData(LocalLightIndex, EyeIndex);
}

FLocalLightData GetLocalLightDataFromGrid(uint GridIndex, uint EyeIndex)
{
	uint LocalLightIndex = GetCulledLightDataGrid(GridIndex);
	return GetLocalLightData(LocalLightIndex, EyeIndex);
}

FLightViewData GetLightViewData(int LightId, uint EyeIndex)
{
	FLightViewData Out = (FLightViewData)0;
	Out = ForwardLightStruct.LightViewData[LightId];

	#if (INSTANCED_STEREO || MOBILE_MULTI_VIEW)
	if (EyeIndex != 0)
	{
		Out.TranslatedWorldPosition.xyz += ForwardLightStruct.PreViewTranslationOffsetISR.xyz;
	}
	#endif

	return Out;
}

/**
 * Helpers to pack/unpack the shadow mask for cluster shading and virtual shadow map
 * Currently hard-coded for 4 bits per light, up to 32 lights per pixel in a uint4
 */
uint4 InitializePackedShadowMask()
{
	return uint(0U).xxxx;
}

uint GetPackedShadowMaskMaxLightCount()
{
	return VirtualShadowMap.PackedShadowMaskMaxLightCount;
}

void PackShadowMask(inout uint4 InOutShadowMask, float InShadowFactor, uint InLightIndex)
{
	uint Value = uint(round((InShadowFactor) * 15.0f)) & 15U;
	uint Dword = InLightIndex / 8;

	InOutShadowMask.x ^= (Dword == 0U) ? (Value << ((InLightIndex      ) * 4U)) : 0U;
	InOutShadowMask.y ^= (Dword == 1U) ? (Value << ((InLightIndex -  8U) * 4U)) : 0U;
	InOutShadowMask.z ^= (Dword == 2U) ? (Value << ((InLightIndex - 16U) * 4U)) : 0U;
	InOutShadowMask.w ^= (Dword == 3U) ? (Value << ((InLightIndex - 24U) * 4U)) : 0U;
}

float UnpackShadowMask(uint4 InShadowMask, uint InLightIndex)
{
	uint Dword = InLightIndex / 8;
	uint MaskBits = (InShadowMask[Dword] >> (InLightIndex - Dword*8U) * 4U) & 15U;
	return (float(MaskBits) / 15.0f);
}

// Unpack with dither to hide some of the quantization
float UnpackShadowMask(uint4 InShadowMask, uint InLightIndex, float Dither)
{	
	float ShadowFactor = UnpackShadowMask(InShadowMask, InLightIndex);
	if (ShadowFactor > 0.0f && ShadowFactor < 1.0f)
	{
		ShadowFactor = saturate(ShadowFactor + (Dither - 0.5f) * (1.0f / 16.0f));
	}
	return ShadowFactor;
}

