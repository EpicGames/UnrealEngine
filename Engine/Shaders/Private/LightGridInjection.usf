// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightGridInjection.usf
=============================================================================*/

#include "Common.ush"
#include "Definitions.usf"
#include "LargeWorldCoordinates.ush"
#include "ReflectionEnvironmentShared.ush"
#include "LightGridCommon.ush"
#include "WaveOpUtil.ush"
#include "GPUMessaging.ush"

// used by USE_HZB_CULL
#include "Nanite/NaniteHZBCull.ush"


#include "/Engine/Shared/LightGridDefinitions.h"
#include "MegaLights/MegaLightsEstimateLight.ush"

RWStructuredBuffer<uint> RWNumCulledLightsGrid;

#if LIGHT_GRID_USES_16BIT_BUFFERS
	RWBuffer<uint>					RWCulledLightDataGrid16Bit;
	#define RWCulledLightDataGrid	RWCulledLightDataGrid16Bit
#else
	RWStructuredBuffer<uint>		RWCulledLightDataGrid32Bit;
	#define RWCulledLightDataGrid	RWCulledLightDataGrid32Bit
#endif

RWStructuredBuffer<uint> RWCulledLightDataAllocator;
RWStructuredBuffer<uint> RWCulledLightLinkAllocator;
RWStructuredBuffer<uint> RWCulledLightLinks;

uint NumReflectionCaptures;
uint NumLocalLights;
uint NumGridCells;
uint MaxCulledLightsPerCell;
uint NumAvailableLinks;
uint3 CulledGridSize;
float3 LightGridZParams;
uint LightGridZSliceScale;
uint LightGridPixelSizeShift;
uint ViewGridCellOffset;
uint ViewCulledDataOffset;

uint LightGridCullMarginXY;
uint LightGridCullMarginZ;
float3 LightGridCullMarginZParams;
uint LightGridCullMaxZ;

uint MegaLightsSupportedStartIndex;
float MinSampleWeightEstimate;

StructuredBuffer<float4> LightViewSpacePositionAndRadius;
StructuredBuffer<float4> LightViewSpaceDirAndPreprocAngle;
StructuredBuffer<float4> LightViewSpaceRectPlanes;
StructuredBuffer<float4> LightViewSpaceWeightEstimate;

StructuredBuffer<int> IndirectionIndices;
UE_DECL(APPLY_INDIRECTION);

// All params used with USE_PARENT_LIGHT_GRID
UE_DECL(USE_PARENT_LIGHT_GRID);
StructuredBuffer<uint> ParentNumCulledLightsGrid;
#if LIGHT_GRID_USES_16BIT_BUFFERS
	Buffer<uint>						ParentCulledLightDataGrid16Bit;
	#define ParentCulledLightDataGrid	ParentCulledLightDataGrid16Bit
#else
	StructuredBuffer<uint>				ParentCulledLightDataGrid32Bit;
	#define ParentCulledLightDataGrid	ParentCulledLightDataGrid32Bit
#endif
uint3 ParentGridSize;
uint NumParentGridCells;
uint ParentGridSizeFactor;


#define NUM_PLANES_PER_RECT_LIGHT 4

#define REFINE_SPOTLIGHT_BOUNDS 1

UE_DECL(REFINE_RECTLIGHT_BOUNDS);

UE_DECL(USE_LINKED_CULL_LIST);

UE_DECL(USE_HZB_CULL);

#ifdef LightGridInjectionCS

float ComputeCellNearViewDepthFromZSlice(uint ZSlice, uint ZSliceScale)
{
	float SliceDepth = ComputeDepthFromZSlice(LightGridZParams, ZSlice * ZSliceScale);

	if (ZSlice == (uint)CulledGridSize.z)
	{
		// Extend the last slice depth max out to world max
		// This allows clamping the depth range to reasonable values, 
		// But has the downside that any lights falling into the last depth slice will have very poor culling,
		// Since the view space AABB will be bloated in x and y
		SliceDepth = 2000000.0f;
	}

	if (ZSlice == 0)
	{
		// The exponential distribution of z slices contains an offset, but some screen pixels
		// may be nearer to the camera than this offset. To avoid false light rejection, we set the
		// first depth slice to zero to ensure that the AABB includes the [0, offset] depth range.
		SliceDepth = View.NearPlane;
	}

	return SliceDepth;
}

// used by USE_HZB_CULL
FScreenRect ComputeCellCullRect(uint3 GridCoordinate, float MinTileZ, float MaxTileZ, float MarginPixels)
{
	const float2 TileSize = (1u << LightGridPixelSizeShift);
	const float2 TileMin = (GridCoordinate.xy + 0) * TileSize - MarginPixels;
	const float2 TileMax = (GridCoordinate.xy + 1) * TileSize + MarginPixels;

	// Compute extent of tiles in clip-space. Note that the last tile may extend a bit outside of view if view size is not evenly divisible tile size.
	const float2 UnitPlaneScale = float2(2.0f, -2.0f) * View.ViewSizeAndInvSize.zw;
	const float2 UnitPlaneBias = float2(-1.0f, 1.0f);

	float2 UnitPlaneTileMin = TileMin * UnitPlaneScale + UnitPlaneBias;
	float2 UnitPlaneTileMax = TileMax * UnitPlaneScale + UnitPlaneBias;

	float MinTileDeviceZ = ConvertToDeviceZ(MinTileZ);
	float MaxTileDeviceZ = ConvertToDeviceZ(MaxTileZ);

	float3 CullRectMin;
	CullRectMin.x = min(UnitPlaneTileMin.x, UnitPlaneTileMax.x);
	CullRectMin.y = min(UnitPlaneTileMin.y, UnitPlaneTileMax.y);
	CullRectMin.z = min(MinTileDeviceZ, MaxTileDeviceZ);

	float3 CullRectMax;
	CullRectMax.x = max(UnitPlaneTileMin.x, UnitPlaneTileMax.x);
	CullRectMax.y = max(UnitPlaneTileMin.y, UnitPlaneTileMax.y);
	CullRectMax.z = max(MinTileDeviceZ, MaxTileDeviceZ);

	return GetScreenRect(int4(0, 0, HZBViewSize), CullRectMin, CullRectMax, 4);
}

struct FCellBounds
{
	float3 ViewAabbMin;
	float3 ViewAabbMax;
	float3 ViewCenter;
	float3 ViewExtent;
};

FCellBounds ComputeCellBounds(uint3 GridCoordinate, float MinTileZ, float MaxTileZ)
{
	// Compute extent of tiles in clip-space. Note that the last tile may extend a bit outside of view if view size is not evenly divisible tile size.
	const float2 InvCulledGridSizeF = (1u << LightGridPixelSizeShift) * View.ViewSizeAndInvSize.zw;
	const float2 TileSize = float2(2.0f, -2.0f) * InvCulledGridSizeF.xy;
	const float2 UnitPlaneMin = float2(-1.0f, 1.0f);

	float2 UnitPlaneTileMin = GridCoordinate.xy * TileSize + UnitPlaneMin;
	float2 UnitPlaneTileMax = (GridCoordinate.xy + 1) * TileSize + UnitPlaneMin;

	float MinTileDeviceZ = ConvertToDeviceZ(MinTileZ);
	float4 MinDepthCorner0 = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MinTileDeviceZ, 1), View.ClipToView);
	float4 MinDepthCorner1 = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MinTileDeviceZ, 1), View.ClipToView);
	float4 MinDepthCorner2 = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMax.y, MinTileDeviceZ, 1), View.ClipToView);
	float4 MinDepthCorner3 = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMin.y, MinTileDeviceZ, 1), View.ClipToView);

	float MaxTileDeviceZ = ConvertToDeviceZ(MaxTileZ);
	float4 MaxDepthCorner0 = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMin.y, MaxTileDeviceZ, 1), View.ClipToView);
	float4 MaxDepthCorner1 = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMax.y, MaxTileDeviceZ, 1), View.ClipToView);
	float4 MaxDepthCorner2 = mul(float4(UnitPlaneTileMin.x, UnitPlaneTileMax.y, MaxTileDeviceZ, 1), View.ClipToView);
	float4 MaxDepthCorner3 = mul(float4(UnitPlaneTileMax.x, UnitPlaneTileMin.y, MaxTileDeviceZ, 1), View.ClipToView);

	float2 ViewMinDepthCorner0 = MinDepthCorner0.xy / MinDepthCorner0.w;
	float2 ViewMinDepthCorner1 = MinDepthCorner1.xy / MinDepthCorner1.w;
	float2 ViewMinDepthCorner2 = MinDepthCorner2.xy / MinDepthCorner2.w;
	float2 ViewMinDepthCorner3 = MinDepthCorner3.xy / MinDepthCorner3.w;
	float2 ViewMaxDepthCorner0 = MaxDepthCorner0.xy / MaxDepthCorner0.w;
	float2 ViewMaxDepthCorner1 = MaxDepthCorner1.xy / MaxDepthCorner1.w;
	float2 ViewMaxDepthCorner2 = MaxDepthCorner2.xy / MaxDepthCorner2.w;
	float2 ViewMaxDepthCorner3 = MaxDepthCorner3.xy / MaxDepthCorner3.w;

	FCellBounds CellBounds;

	//@todo - derive min and max from quadrant
	CellBounds.ViewAabbMin.xy = min(ViewMinDepthCorner0, ViewMinDepthCorner1);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMinDepthCorner2);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMinDepthCorner3);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMaxDepthCorner0);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMaxDepthCorner1);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMaxDepthCorner2);
	CellBounds.ViewAabbMin.xy = min(CellBounds.ViewAabbMin.xy, ViewMaxDepthCorner3);
	CellBounds.ViewAabbMax.xy = max(ViewMinDepthCorner0, ViewMinDepthCorner1);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMinDepthCorner2);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMinDepthCorner3);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMaxDepthCorner0);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMaxDepthCorner1);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMaxDepthCorner2);
	CellBounds.ViewAabbMax.xy = max(CellBounds.ViewAabbMax.xy, ViewMaxDepthCorner3);
	CellBounds.ViewAabbMin.z = MinTileZ;
	CellBounds.ViewAabbMax.z = MaxTileZ;
	CellBounds.ViewCenter = .5f * (CellBounds.ViewAabbMin + CellBounds.ViewAabbMax);
	CellBounds.ViewExtent = CellBounds.ViewAabbMax - CellBounds.ViewCenter;

	return CellBounds;
}

bool IntersectConeWithSphere(float3 ConeVertex, float3 ConeAxis, float ConeRadius, float2 CosSinAngle, float4 SphereToTest)
{
    float3 ConeVertexToSphereCenter = SphereToTest.xyz - ConeVertex;
    float ConeVertexToSphereCenterLengthSq = dot(ConeVertexToSphereCenter, ConeVertexToSphereCenter);
    float SphereProjectedOntoConeAxis = dot(ConeVertexToSphereCenter, -ConeAxis);
    float DistanceToClosestPoint = CosSinAngle.x * sqrt(ConeVertexToSphereCenterLengthSq - SphereProjectedOntoConeAxis * SphereProjectedOntoConeAxis) - SphereProjectedOntoConeAxis * CosSinAngle.y;
 
    bool bSphereTooFarFromCone = DistanceToClosestPoint > SphereToTest.w;
    bool bSpherePastConeEnd = SphereProjectedOntoConeAxis > SphereToTest.w + ConeRadius;
    bool bSphereBehindVertex = SphereProjectedOntoConeAxis < -SphereToTest.w;
	return !(bSphereTooFarFromCone || bSpherePastConeEnd || bSphereBehindVertex);
}

/**
 * Returns true if the aabb defined by Center and Extents is fully in front of the plane.
 */
bool IsAabbInFrontOfPlane(float3 Center, float3 Extents, float4 Plane)
{
	float Dist = dot(float4(Center, 1.0), Plane);
	float Radius = dot(Extents, abs(Plane.xyz));
	return Dist > Radius;
}

/**
 * Approximate cone / aabb test that creates a single separating plane that lies in the cone on the side facing the centre of the Aabb
 * Returns false if the Aabb is outside (entirely on the positive side) of this plane.
 * Returns true otherwise, only works for 'acute angled' cones, where the angle is < 90 degrees.
 * Is approximate, in that it can yield false negatives, i.e., that an Aabb may be actually outside, but the test still returns false.
 * Since the intended use is to cull light cones, this is acceptable whereas false positives would cause glitches.
 */
bool IsAabbOutsideInfiniteAcuteConeApprox(float3 ConeVertex, float3 ConeAxis, float TanConeAngle, float3 AabbCentre, float3 AabbExt)
{
	// 1. find plane (well, base) in which normal lies, and which is perpendicular to axis and centre of aabb.
	float3 D = AabbCentre - ConeVertex;

	// perpendicular to cone axis in plane of cone axis and aabb centre.
	float3 M = -normalize(cross(cross(D, ConeAxis), ConeAxis));
	float3 N = -TanConeAngle * ConeAxis + M;
	float4 Plane = float4(N, 0.0);

	return IsAabbInFrontOfPlane(D, AabbExt, Plane);
}

bool OverlapsLight(uint LocalLightIndex, FCellBounds CellBounds, out bool bIsRectLight, out bool bIsTexturedLight)
{
	checkSlow(LocalLightIndex < NumLocalLights);

	float4 LightPositionAndRadius = LightViewSpacePositionAndRadius[LocalLightIndex];
	float3 ViewSpaceLightPosition = LightPositionAndRadius.xyz;
	float LightRadius = LightPositionAndRadius.w;
	float BoxDistanceSq = ComputeSquaredDistanceFromBoxToPoint(CellBounds.ViewCenter, CellBounds.ViewExtent, ViewSpaceLightPosition);
	if (BoxDistanceSq < LightRadius * LightRadius)
	{
		float4 ViewSpaceDirAndPreprocAngle = LightViewSpaceDirAndPreprocAngle[LocalLightIndex];
		bIsRectLight = asuint(ViewSpaceDirAndPreprocAngle.w) & 0x1;
		bool bUseTightRectLightCulling = asuint(ViewSpaceDirAndPreprocAngle.w) & 0x2;
		bIsTexturedLight = asuint(ViewSpaceDirAndPreprocAngle.w) & 0x4;

		bool bPassEstimatedWeightTest = true;

		if (LocalLightIndex >= MegaLightsSupportedStartIndex)
		{
			const float4 LightWeightEstimate = LightViewSpaceWeightEstimate[LocalLightIndex];
			const float LightPower = LightWeightEstimate.x;
			const float LightFalloffExponent = LightWeightEstimate.y;

			float EstimatedWeight = EstimateLocalLight(
				BoxDistanceSq,
				LightPower,
				1.0f / LightRadius,	
				LightFalloffExponent);

			bPassEstimatedWeightTest = EstimatedWeight > MinSampleWeightEstimate;
		}

		bool bPassSpotLightTest = true;
		bool bPassRectLightTest = true;

		if (bPassEstimatedWeightTest)
		{
			#if REFINE_SPOTLIGHT_BOUNDS
			{
				float TanConeAngle = asfloat(asuint(ViewSpaceDirAndPreprocAngle.w) & 0xFFFFFFF8);

				// Set to 0 for non-acute cones, or non-spot lights.
				if (TanConeAngle > 0.0f)
				{
					float3 ViewSpaceLightDirection = -ViewSpaceDirAndPreprocAngle.xyz;
					bPassSpotLightTest = !IsAabbOutsideInfiniteAcuteConeApprox(ViewSpaceLightPosition, ViewSpaceLightDirection, TanConeAngle, CellBounds.ViewCenter, CellBounds.ViewExtent);
				}
			}
			#endif // REFINE_SPOTLIGHT_BOUNDS

			if(UE_EVAL(REFINE_RECTLIGHT_BOUNDS))
			{
				if (bIsRectLight)
				{
					float3 D = CellBounds.ViewCenter - ViewSpaceLightPosition;
					float4 Plane = float4(ViewSpaceDirAndPreprocAngle.xyz, 0.0);

					bPassRectLightTest = !IsAabbInFrontOfPlane(D, CellBounds.ViewExtent, Plane);

					if(bPassRectLightTest && bUseTightRectLightCulling)
					{
						for (uint PlaneIndex = 0; PlaneIndex < NUM_PLANES_PER_RECT_LIGHT; PlaneIndex++)
						{
							if (IsAabbInFrontOfPlane(CellBounds.ViewCenter, CellBounds.ViewExtent, LightViewSpaceRectPlanes[LocalLightIndex * NUM_PLANES_PER_RECT_LIGHT + PlaneIndex]))
							{
								bPassRectLightTest = false;
							}
						}
					}
				}
			}
		}

		return bPassEstimatedWeightTest && bPassSpotLightTest && bPassRectLightTest;
	}


	bIsRectLight = false;
	bIsTexturedLight = false;

	return false;
}

bool OverlapsReflectionCapture(uint ReflectionCaptureIndex, FCellBounds CellBounds)
{
	checkSlow(ReflectionCaptureIndex < NumReflectionCaptures);

	FDFVector3 CaptureWorldPosition = MakeDFVector3(GetReflectionPositionAndRadius(ReflectionCaptureIndex).xyz, GetReflectionPositionLow(ReflectionCaptureIndex).xyz);
	float3 CaptureTranslatedWorldPosition = DFFastAddDemote(CaptureWorldPosition, PrimaryView.PreViewTranslation);
	float3 ViewSpaceCapturePosition = mul(float4(CaptureTranslatedWorldPosition, 1), View.TranslatedWorldToView).xyz;
	float CaptureRadius = GetReflectionPositionAndRadius(ReflectionCaptureIndex).w;

	float BoxDistanceSq = ComputeSquaredDistanceFromBoxToPoint(CellBounds.ViewCenter, CellBounds.ViewExtent, ViewSpaceCapturePosition);

	return (BoxDistanceSq < CaptureRadius * CaptureRadius);
}

void OutputReversedLinkedList(uint LinkOffset, uint NumPrimitives, uint CulledLightDataStart, bool bApplyIndirection)
{
	uint CurrentIndex = 0;

	while (LinkOffset != 0xFFFFFFFF && CurrentIndex < NumPrimitives)
	{
		// Reverse the order as we write them out, which restores the original order before the reverse linked list was built
		uint PrimitiveIndex = RWCulledLightLinks[LinkOffset * LIGHT_LINK_STRIDE + 0];

		if (bApplyIndirection)
		{
			PrimitiveIndex = IndirectionIndices[PrimitiveIndex];
		}

		RWCulledLightDataGrid[CulledLightDataStart + NumPrimitives - CurrentIndex - 1] = PrimitiveIndex;
		CurrentIndex++;
		LinkOffset = RWCulledLightLinks[LinkOffset * LIGHT_LINK_STRIDE + 1];
	}
}

struct FArrayView
{
	uint Num;
	uint Offset;
};

uint GetPrimitiveIndex(FArrayView ArrayView, uint Index)
{
	checkSlow(Index < ArrayView.Num);
	if (UE_EVAL(USE_PARENT_LIGHT_GRID))
	{
		return ParentCulledLightDataGrid[ArrayView.Offset + Index];
	}
	else
	{
		return ArrayView.Offset + Index;
	}
}

FArrayView InitLocalLightArrayView(uint ParentGridIndex)
{
	FArrayView ArrayView;

	if (UE_EVAL(USE_PARENT_LIGHT_GRID))
	{
		uint Unused_NumVisibleMegaLights;
		UnpackCulledLightsGridHeader0(ParentNumCulledLightsGrid[ParentGridIndex * 2 + 0], ArrayView.Num, Unused_NumVisibleMegaLights);

		bool bHasRectLights_Unused;
		bool bHasTexturedLights_Unused;
		UnpackCulledLightsGridHeader1(ParentNumCulledLightsGrid[ParentGridIndex * 2 + 1], ArrayView.Offset, bHasRectLights_Unused, bHasTexturedLights_Unused);
	}
	else
	{
		ArrayView.Num = NumLocalLights;
		ArrayView.Offset = 0;
	}

	return ArrayView;
}

FArrayView InitReflectionCaptureArrayView(uint ParentGridIndex)
{
	FArrayView ArrayView;

	if (UE_EVAL(USE_PARENT_LIGHT_GRID))
	{
		ArrayView.Num = ParentNumCulledLightsGrid[(NumParentGridCells + ParentGridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 0];
		ArrayView.Offset = ParentNumCulledLightsGrid[(NumParentGridCells + ParentGridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 1];
	}
	else
	{
		ArrayView.Num = NumReflectionCaptures;
		ArrayView.Offset = 0;
	}

	return ArrayView;
}

void CullLights_SingleThread(uint GridIndex, FCellBounds CellBounds, uint ParentGridIndex)
{
	uint NumVisibleLights = 0;
	uint NumVisibleMegaLights = 0;
	uint bHasRectLights = false;
	uint bHasTexturedLights = false;

	uint LinkOffset = 0xFFFFFFFF;

	FArrayView LocalLightArrayView = InitLocalLightArrayView(ParentGridIndex);

	LOOP
	for (uint Index = 0; Index < LocalLightArrayView.Num; ++Index)
	{
		uint LocalLightIndex = GetPrimitiveIndex(LocalLightArrayView, Index);

		bool bIsRectLight;
		bool bIsTexturedLight;
		bool bOverlapsLight = OverlapsLight(LocalLightIndex, CellBounds, bIsRectLight, bIsTexturedLight);

		if (bOverlapsLight)
		{
			bHasRectLights |= bIsRectLight;
			bHasTexturedLights |= bIsTexturedLight;

			if (UE_EVAL(USE_LINKED_CULL_LIST))
			{
				uint NextLink;
				WaveInterlockedAddScalar_(RWCulledLightLinkAllocator[0], 1U, NextLink);

				if (NextLink < NumAvailableLinks)
				{
					RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 0] = LocalLightIndex;
					RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 1] = LinkOffset;

					LinkOffset = NextLink;

					++NumVisibleLights;
					if (LocalLightIndex >= MegaLightsSupportedStartIndex)
					{
						++NumVisibleMegaLights;
					}
				}
			}
			else
			{
				if (NumVisibleLights < MaxCulledLightsPerCell)
				{
					uint PrimitiveIndex = LocalLightIndex;

					if (UE_EVAL(APPLY_INDIRECTION))
					{
						PrimitiveIndex = IndirectionIndices[PrimitiveIndex];
					}

					RWCulledLightDataGrid[GridIndex * MaxCulledLightsPerCell + NumVisibleLights] = PrimitiveIndex;

					++NumVisibleLights;
					if (LocalLightIndex >= MegaLightsSupportedStartIndex)
					{
						++NumVisibleMegaLights;
					}
				}
			}
		}
	}

	uint CulledLightDataStart;
	if (UE_EVAL(USE_LINKED_CULL_LIST))
	{
		#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
			// Adreno compiler fails to reg-alloc on 6xx for this shader,
			// and doesn't support UGPR spilling, so just fails to compile.
			InterlockedAdd(RWCulledLightDataAllocator[0], NumVisibleLights, CulledLightDataStart);
		#else
			WaveInterlockedAdd_(RWCulledLightDataAllocator[0], NumVisibleLights, CulledLightDataStart);
		#endif

		CulledLightDataStart += ViewCulledDataOffset;

		// Mega Lights are order dependent
		OutputReversedLinkedList(LinkOffset, NumVisibleLights, CulledLightDataStart, UE_EVAL(APPLY_INDIRECTION));
	}
	else
	{
		CulledLightDataStart = GridIndex * MaxCulledLightsPerCell;
	}

	RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = PackCulledLightsGridHeader0(NumVisibleLights, NumVisibleMegaLights);
	RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = PackCulledLightsGridHeader1(CulledLightDataStart, bHasRectLights, bHasTexturedLights);
}
void CullReflectionCaptures_SingleThread(uint GridIndex, FCellBounds CellBounds, uint ParentGridIndex)
{
	uint NumVisibleCaptures = 0;

	uint LinkOffset = 0xFFFFFFFF;

	FArrayView ReflectionCaptureArrayView = InitReflectionCaptureArrayView(ParentGridIndex);

	LOOP
	for (uint Index = 0; Index < ReflectionCaptureArrayView.Num; ++Index)
	{
		uint ReflectionCaptureIndex = GetPrimitiveIndex(ReflectionCaptureArrayView, Index);

		bool bOverlapsReflectionCapture = OverlapsReflectionCapture(ReflectionCaptureIndex, CellBounds);

		if (bOverlapsReflectionCapture)
		{
			if (UE_EVAL(USE_LINKED_CULL_LIST))
			{
				uint NextLink;
				WaveInterlockedAddScalar_(RWCulledLightLinkAllocator[0], 1U, NextLink);

				if (NextLink < NumAvailableLinks)
				{
					RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 0] = ReflectionCaptureIndex;
					RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 1] = LinkOffset;

					LinkOffset = NextLink;

					++NumVisibleCaptures;
				}
			}
			else
			{
				if (NumVisibleCaptures < MaxCulledLightsPerCell)
				{
					RWCulledLightDataGrid[(NumGridCells + GridIndex) * MaxCulledLightsPerCell + NumVisibleCaptures] = ReflectionCaptureIndex;

					++NumVisibleCaptures;
				}
			}
		}
	}

	uint CulledReflectionCaptureDataStart;
	if (UE_EVAL(USE_LINKED_CULL_LIST))
	{
		WaveInterlockedAdd_(RWCulledLightDataAllocator[0], NumVisibleCaptures, CulledReflectionCaptureDataStart);

		CulledReflectionCaptureDataStart += ViewCulledDataOffset;

		// Reflection captures are order dependent
		OutputReversedLinkedList(LinkOffset, NumVisibleCaptures, CulledReflectionCaptureDataStart, false);
	}
	else
	{
		CulledReflectionCaptureDataStart = (NumGridCells + GridIndex) * MaxCulledLightsPerCell;
	}

	RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = NumVisibleCaptures;
	RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = CulledReflectionCaptureDataStart;
}

#define NUM_THREADS_PER_GROUP THREADGROUP_SIZE
#include "ThreadGroupPrefixSum.ush"

// When using one thread group per cell, store up to MAX_LDS_ITEMS in LDS before falling back to linked list
#define USE_LDS (1)
#define MAX_LDS_ITEMS 256

groupshared uint CellLDSItems[MAX_LDS_ITEMS];

struct FCellWriter
{
	uint LinkOffset;
};

FCellWriter InitCellWriter()
{
	FCellWriter CellWriter;
	CellWriter.LinkOffset = LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET;
	
	return CellWriter;
}

bool AddToCell(inout FCellWriter CellWriter, uint PrimitiveIndex, uint IndexInCell)
{
#if USE_LDS
	// use LDS if there's still space
	if (IndexInCell < MAX_LDS_ITEMS)
	{
		CellLDSItems[IndexInCell] = PrimitiveIndex;
		return true;
	}
#endif

	// otherwise store in linked list
	uint NextLink;
	WaveInterlockedAddScalar_(RWCulledLightLinkAllocator[0], 1U, NextLink);

	if (NextLink < NumAvailableLinks)
	{
		checkSlow(PrimitiveIndex <= LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET);
		checkSlow(IndexInCell <= 0xFFFF);
		checkSlow(CellWriter.LinkOffset <= LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET);

		uint IndexInCellL8 = (IndexInCell & 0xFF);
		uint IndexInCellH8 = ((IndexInCell >> 8) & 0xFF);

		RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 0] = (IndexInCellL8 << 24) | (PrimitiveIndex & LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET);
		RWCulledLightLinks[NextLink * LIGHT_LINK_STRIDE + 1] = (IndexInCellH8 << 24) | (CellWriter.LinkOffset & LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET);

		CellWriter.LinkOffset = NextLink;

		return true;
	}

	return false;
}

void OutputCell(FCellWriter CellWriter, uint NumPrimitives, uint CulledLightDataStart, uint ThreadIndex, bool bApplyIndirection)
{
#if USE_LDS
	// output entries in LDS to RWCulledLightDataGrid
	LOOP
	for (uint OutIndex = ThreadIndex; OutIndex < NumPrimitives; OutIndex += THREADGROUP_SIZE)
	{
		uint PrimitiveIndex = CellLDSItems[OutIndex];

		if (bApplyIndirection)
		{
			PrimitiveIndex = IndirectionIndices[PrimitiveIndex];
		}

		RWCulledLightDataGrid[CulledLightDataStart + OutIndex] = PrimitiveIndex;
	}
#endif

	// output entries in linked list to RWCulledLightDataGrid

	while (CellWriter.LinkOffset < LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET)
	{
		const uint PackedData0 = RWCulledLightLinks[CellWriter.LinkOffset * LIGHT_LINK_STRIDE + 0];
		const uint PackedData1 = RWCulledLightLinks[CellWriter.LinkOffset * LIGHT_LINK_STRIDE + 1];
		uint PrimitiveIndex = (PackedData0 & 0xFFFFFF);
		const uint LinkOffset = (PackedData1 & 0xFFFFFF);
		const uint IndexInCellL8 = (PackedData0 >> 24) & 0xFF;
		const uint IndexInCellH8 = (PackedData1 >> 24) & 0xFF;
		const uint IndexInCell = (IndexInCellH8 << 8) | IndexInCellL8;

		if (UE_EVAL(APPLY_INDIRECTION))
		{
			PrimitiveIndex = IndirectionIndices[PrimitiveIndex];
		}

		RWCulledLightDataGrid[CulledLightDataStart + IndexInCell] = PrimitiveIndex;
		CellWriter.LinkOffset = LinkOffset;
	}
}

groupshared uint OutOffset;
groupshared uint bCellHasRectLights;
groupshared uint bCellHasTexturedLights;
groupshared uint CellNumVisibleLights;
groupshared uint CellNumVisibleMegaLights;
groupshared uint CellNumVisibleReflectionCaptures;

groupshared uint CulledLightDataStart;

UE_DECL(USE_THREAD_GROUP_PER_CELL);

void CullLights_ThreadGroup(uint ThreadIndex, uint GridIndex, FCellBounds CellBounds, uint ParentGridIndex)
{
	if (ThreadIndex == 0)
	{
		OutOffset = 0;
		bCellHasRectLights = 0;
		bCellHasTexturedLights = 0;
		CellNumVisibleLights = 0;
		CellNumVisibleMegaLights = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	uint NumVisibleLights = 0;
	uint NumVisibleMegaLights = 0;
	uint bHasRectLights = false;
	uint bHasTexturedLights = false;

	FCellWriter CellWriter = InitCellWriter();

	FArrayView LocalLightArrayView = InitLocalLightArrayView(ParentGridIndex);

	LOOP
	for (uint GroupBaseIndex = 0; GroupBaseIndex < LocalLightArrayView.Num; GroupBaseIndex += THREADGROUP_SIZE)
	{
		const uint Index = GroupBaseIndex + ThreadIndex;
		const uint LocalLightIndex = Index < LocalLightArrayView.Num ? GetPrimitiveIndex(LocalLightArrayView, Index) : 0xFFFFFFFF;

		bool bIsRectLight;
		bool bIsTexturedLight;
		bool bOverlapsLight = LocalLightIndex < NumLocalLights;
		if (bOverlapsLight)
		{
			bOverlapsLight = OverlapsLight(LocalLightIndex, CellBounds, bIsRectLight, bIsTexturedLight);
		}
		
		// NOTE: Cannot be under any divergent branching!
		uint GroupCount;
		uint Offset = ThreadGroupPrefixSum(bOverlapsLight ? 1u : 0u, ThreadIndex, GroupCount);

		uint IndexInCell = OutOffset + Offset;

		// Wait until all threads are done with 'OutOffset'
		GroupMemoryBarrierWithGroupSync();
		
		if (ThreadIndex == 0)
		{
			OutOffset += GroupCount;
		}

		if (bOverlapsLight)
		{
			bHasRectLights |= bIsRectLight;
			bHasTexturedLights |= bIsTexturedLight;

			if (AddToCell(CellWriter, LocalLightIndex, IndexInCell))
			{
				++NumVisibleLights;
				if (LocalLightIndex >= MegaLightsSupportedStartIndex)
				{
					++NumVisibleMegaLights;
				}
			}
		}
	}

	InterlockedOr(bCellHasRectLights, bHasRectLights ? 1 : 0);
	InterlockedOr(bCellHasTexturedLights, bHasTexturedLights ? 1 : 0);
	InterlockedAdd(CellNumVisibleLights, NumVisibleLights);
	InterlockedAdd(CellNumVisibleMegaLights, NumVisibleMegaLights);

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex == 0)
	{
		InterlockedAdd(RWCulledLightDataAllocator[0], CellNumVisibleLights, CulledLightDataStart);
	}

	GroupMemoryBarrierWithGroupSync();

	OutputCell(CellWriter, CellNumVisibleLights, CulledLightDataStart, ThreadIndex, UE_EVAL(APPLY_INDIRECTION));

	if (ThreadIndex == 0)
	{
		RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = PackCulledLightsGridHeader0(CellNumVisibleLights, CellNumVisibleMegaLights);
		RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = PackCulledLightsGridHeader1(CulledLightDataStart + ViewCulledDataOffset, bCellHasRectLights, bCellHasTexturedLights);
	}
}

void CullReflectionCaptures_ThreadGroup(uint ThreadIndex, uint GridIndex, FCellBounds CellBounds, uint ParentGridIndex)
{
	if (ThreadIndex == 0)
	{
		OutOffset = 0;
		CellNumVisibleReflectionCaptures = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	uint NumVisibleCaptures = 0;

	FCellWriter CellWriter = InitCellWriter();

	FArrayView ReflectionCaptureArrayView = InitReflectionCaptureArrayView(ParentGridIndex);

	LOOP
	for (uint GroupBaseIndex = 0; GroupBaseIndex < ReflectionCaptureArrayView.Num; GroupBaseIndex += THREADGROUP_SIZE)
	{
		const uint Index = GroupBaseIndex + ThreadIndex;
		const uint ReflectionCaptureIndex = Index < ReflectionCaptureArrayView.Num ? GetPrimitiveIndex(ReflectionCaptureArrayView, Index) : 0xFFFFFFFF;

		bool bOverlapsReflectionCapture = ReflectionCaptureIndex < NumReflectionCaptures;
		if (bOverlapsReflectionCapture)
		{
			bOverlapsReflectionCapture = OverlapsReflectionCapture(ReflectionCaptureIndex, CellBounds);
		}
		
		// NOTE: Cannot be under any divergent branching!
		uint GroupCount;
		uint Offset = ThreadGroupPrefixSum(bOverlapsReflectionCapture ? 1u : 0u, ThreadIndex, GroupCount);
		
		uint IndexInCell = OutOffset + Offset;

		// Wait until all threads are done with 'OutOffset'
		GroupMemoryBarrierWithGroupSync();
		
		if (ThreadIndex == 0)
		{
			OutOffset += GroupCount;
		}

		if (bOverlapsReflectionCapture)
		{
			if (AddToCell(CellWriter, ReflectionCaptureIndex, IndexInCell))
			{
				++NumVisibleCaptures;
			}
		}
	}

	InterlockedAdd(CellNumVisibleReflectionCaptures, NumVisibleCaptures);

	GroupMemoryBarrierWithGroupSync();

	if (ThreadIndex == 0)
	{
		InterlockedAdd(RWCulledLightDataAllocator[0], CellNumVisibleReflectionCaptures, CulledLightDataStart);
	}

	GroupMemoryBarrierWithGroupSync();

	OutputCell(CellWriter, CellNumVisibleReflectionCaptures, CulledLightDataStart, ThreadIndex, false);

	if (ThreadIndex == 0)
	{
		RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = CellNumVisibleReflectionCaptures;
		RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = CulledLightDataStart + ViewCulledDataOffset;
	}
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void LightGridInjectionCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupId : SV_GroupID,
	uint GroupIndex : SV_GroupIndex)
{
	const uint ThreadIndex = GroupIndex;
	const uint3 GridCoordinate = (UE_EVAL(USE_THREAD_GROUP_PER_CELL)) ? GroupId : DispatchThreadId;

	if (all(GridCoordinate < (uint3)CulledGridSize))
	{
		const uint GridIndex = (GridCoordinate.z * CulledGridSize.y + GridCoordinate.y) * CulledGridSize.x + GridCoordinate.x + ViewGridCellOffset;

// Disable to pass all lights through for debugging, will hit limits quickly though
#define CULL_LIGHTS 1
#if CULL_LIGHTS

		float MinTileZ = ComputeCellNearViewDepthFromZSlice(GridCoordinate.z + 0, LightGridZSliceScale);
		float MaxTileZ = ComputeCellNearViewDepthFromZSlice(GridCoordinate.z + 1, LightGridZSliceScale);

		if (UE_EVAL(USE_HZB_CULL))
		{
			float CullMinTileZ = MinTileZ;
			float CullMaxTileZ = MaxTileZ;
			if(LightGridCullMarginZ > 0)
			{
				// convert Min/MaxTileZ to volumetric fog grid coordinate
				int MinCullSlice = (int)floor(ComputeZSliceFromDepth(LightGridCullMarginZParams, MinTileZ));
				int MaxCullSlice = (int)ceil(ComputeZSliceFromDepth(LightGridCullMarginZParams, MaxTileZ));
				// add margins
				MinCullSlice = max(0, MinCullSlice - (int)LightGridCullMarginZ);
				MaxCullSlice = min(LightGridCullMaxZ, MaxCullSlice + LightGridCullMarginZ);
				// convert back to Min/MaxTileZ
				CullMinTileZ = min(CullMinTileZ, ComputeDepthFromZSlice(LightGridCullMarginZParams, MinCullSlice));
				CullMaxTileZ = max(CullMaxTileZ, ComputeDepthFromZSlice(LightGridCullMarginZParams, MaxCullSlice));
			}

			FScreenRect Rect = ComputeCellCullRect(GridCoordinate, CullMinTileZ, CullMaxTileZ, LightGridCullMarginXY);

 			if (!IsVisibleHZB(Rect, true /*bSample4x4*/))
 			{
				if (!UE_EVAL(USE_THREAD_GROUP_PER_CELL) || ThreadIndex == 0)
				{
 					RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = PackCulledLightsGridHeader0(0, 0);
 					RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = PackCulledLightsGridHeader1(0, false, false);
				}
 				return;
 			}
		
			// tighten cell bounds based on farthest depth in cell
			{
				const float FarthestDeviceZ = GetMinDepthFromHZB(Rect, true /*bSample4x4*/);
				const float FarthestSceneDepth = ConvertFromDeviceZ(FarthestDeviceZ);

				MaxTileZ = min(MaxTileZ, FarthestSceneDepth);
			}
		}
		
		FCellBounds CellBounds = ComputeCellBounds(GridCoordinate, MinTileZ, MaxTileZ);

		uint ParentGridIndex;
		if (UE_EVAL(USE_PARENT_LIGHT_GRID))
		{
			const uint3 ParentGridCoordinate = GridCoordinate / ParentGridSizeFactor;
			ParentGridIndex = (ParentGridCoordinate.z * ParentGridSize.y + ParentGridCoordinate.y) * ParentGridSize.x + ParentGridCoordinate.x;
		}
		else
		{
			ParentGridIndex = 0xFFFFFFFF;
		}

		if (UE_EVAL(USE_THREAD_GROUP_PER_CELL))
		{
			CullLights_ThreadGroup(ThreadIndex, GridIndex, CellBounds, ParentGridIndex);
			CullReflectionCaptures_ThreadGroup(ThreadIndex, GridIndex, CellBounds, ParentGridIndex);
		}
		else
		{
			CullLights_SingleThread(GridIndex, CellBounds, ParentGridIndex);
			CullReflectionCaptures_SingleThread(GridIndex, CellBounds, ParentGridIndex);
		}

#else // !CULL_LIGHTS
		LOOP
		for (uint LocalLightIndex = 0; LocalLightIndex < NumLocalLights; LocalLightIndex++)
		{
			if (LocalLightIndex < MaxCulledLightsPerCell)
			{
				RWCulledLightDataGrid[GridIndex * MaxCulledLightsPerCell + LocalLightIndex] = LocalLightIndex;
			}
		}

		const uint NumVisibleLights = min(NumLocalLights, MaxCulledLightsPerCell);
		const uint NumVisibleMegaLights = max(NumVisibleLights - MegaLightsSupportedStartIndex, 0);
		RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = PackCulledLightsGridHeader0(NumVisibleLights, NumVisibleMegaLights);
		RWNumCulledLightsGrid[GridIndex * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = PackCulledLightsGridHeader1(GridIndex * MaxCulledLightsPerCell, true, true); // TODO
		
		LOOP
		for (uint ReflectionCaptureIndex = 0; ReflectionCaptureIndex < NumReflectionCaptures; ReflectionCaptureIndex++)
		{
			if (ReflectionCaptureIndex < MaxCulledLightsPerCell)
			{
				RWCulledLightDataGrid[(NumGridCells + GridIndex) * MaxCulledLightsPerCell + ReflectionCaptureIndex] = ReflectionCaptureIndex;
			}
		}

		const uint NumVisibleReflectionCaptures = min(NumReflectionCaptures, MaxCulledLightsPerCell);
		RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 0] = NumVisibleReflectionCaptures;
		RWNumCulledLightsGrid[(NumGridCells + GridIndex) * NUM_CULLED_LIGHTS_GRID_STRIDE + 1] = (NumGridCells + GridIndex) * MaxCulledLightsPerCell;
#endif // !CULL_LIGHTS
	}
}

#endif // LightGridInjectionCS


#ifdef SHADER_DEBUG_LIGHT_GRID_PS

#define SUPPORT_CONTACT_SHADOWS 0
#include "/Engine/Private/ShaderPrint.ush"
#include "ColorMap.ush"
#include "ScreenPass.ush"

Texture2D<float4> DepthTexture;
FScreenTransform ScreenToPrimaryScreenPos;
uint DebugMode;
uint MaxThreshold;

void DebugLightGridPS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminanceTransmittance : SV_Target0)
{
	OutLuminanceTransmittance = float4(0, 0, 0, 1);

	ResolvedView = ResolveView();

	const uint EyeIndex = 0;
	const uint2 PixelPos = SVPos.xy;

	const float2 PixelPosDynRes = ApplyScreenTransform(PixelPos, ScreenToPrimaryScreenPos);
	const float2 PixelPosDynRes2 = ApplyScreenTransform(PixelPos + 1, ScreenToPrimaryScreenPos);
	const uint2 LocalPixelPosDynRes = (PixelPosDynRes.xy - ResolvedView.ViewRectMin.xy);
	const uint2 LocalPixelPosDynRes2 = (PixelPosDynRes2.xy - ResolvedView.ViewRectMin.xy);

	if (any(LocalPixelPosDynRes >= uint2(ResolvedView.ViewSizeAndInvSize.xy)))
	{
		return;
	}

	const FLightGridData GridData = GetLightGridData();
	uint DebugNumLightsInGridCell = 0;

	uint2 StartPos = uint2(50, 100);
	FShaderPrintContext Context = InitShaderPrintContext(all(PixelPos == StartPos), StartPos);
	bool SpecifyDebugSlice = AddCheckbox(Context, TEXT("Specify Slice to debug"), false, GetDefaultFontColor());
	Newline(Context);
	float DebugSlice = 0;
	FFontColor EnableSliceDebugFont = FontDarkGrey; if (SpecifyDebugSlice) { EnableSliceDebugFont = FontWhite; }
	DebugSlice = AddSlider(Context, TEXT("Slice to debug"), 0.0f, EnableSliceDebugFont, 0.0f, GridData.CulledGridSize.z-1);
	Newline(Context);
	bool bIsolateMegaLights = AddCheckbox(Context, TEXT("Isolate MegaLights"), false, GetDefaultFontColor());
	Newline(Context);
	bool bExcludeMegaLights = AddCheckbox(Context, TEXT("Exclude MegaLights"), false, GetDefaultFontColor());
	Newline(Context);

	if (DebugMode == 1 && !SpecifyDebugSlice)
	{
		const float SceneDepth = ConvertFromDeviceZ(DepthTexture.Load(int3(PixelPosDynRes, 0)).r);

		const uint GridIndex = ComputeLightGridCellIndex(LocalPixelPosDynRes, SceneDepth, EyeIndex);
		const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

		DebugNumLightsInGridCell = bIsolateMegaLights ? CulledLightsGridHeader.NumMegaLights : (CulledLightsGridHeader.NumLights - (bExcludeMegaLights ? CulledLightsGridHeader.NumMegaLights : 0));
	}
	else
	{
		// We do not want to fetch light data from the last slice since in this case the culling will be pretty bad (super large depth turned into a bounding sphere...)
		// See ComputeCellNearViewDepthFromZSlice.
		const int StartGridIndexZ = SpecifyDebugSlice ? uint(DebugSlice) : 0;
		const int EndGridIndexZ   = SpecifyDebugSlice ? StartGridIndexZ+1: (DebugMode == 2 ? GridData.CulledGridSize.z - 1 : GridData.CulledGridSize.z);
		LOOP
		for (int SliceIt = StartGridIndexZ; SliceIt < EndGridIndexZ; SliceIt++)
		{
			const uint GridIndex = ComputeLightGridCellIndex(uint3(LocalPixelPosDynRes >> GridData.LightGridPixelSizeShift, SliceIt), EyeIndex);
			const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);
		
			const uint NumLightsInSlice = DebugNumLightsInGridCell = bIsolateMegaLights ? CulledLightsGridHeader.NumMegaLights : (CulledLightsGridHeader.NumLights - (bExcludeMegaLights ? CulledLightsGridHeader.NumMegaLights : 0));
			DebugNumLightsInGridCell = max(DebugNumLightsInGridCell, NumLightsInSlice);
		}
	}

	const uint2 TileTopLeftPixelCoord = (LocalPixelPosDynRes  >> GridData.LightGridPixelSizeShift) << GridData.LightGridPixelSizeShift;
	const uint2 TileTopLeftPixelCoord2= (LocalPixelPosDynRes2 >> GridData.LightGridPixelSizeShift) << GridData.LightGridPixelSizeShift;
	const uint TileSize = (0x1u << GridData.LightGridPixelSizeShift) - 1;

	if (DebugNumLightsInGridCell > 0)
	{
		OutLuminanceTransmittance.a   = 0.0f;
		OutLuminanceTransmittance.rgb = GetHSVDebugColor(float(DebugNumLightsInGridCell) / MaxThreshold);

		int2 TextTopLeftPosition = (TileTopLeftPixelCoord + TileSize / 2);
		TextTopLeftPosition = (TextTopLeftPosition - ScreenToPrimaryScreenPos.zw) / ScreenToPrimaryScreenPos.xy; // invert ScreenToPrimaryScreenPos transform

		PrintSmallUint(PixelPos, OutLuminanceTransmittance.rgb, float3(0.1, 0.1, 0.1), TextTopLeftPosition, float(DebugNumLightsInGridCell));
	}

	if ((TileTopLeftPixelCoord.x <= LocalPixelPosDynRes.x && TileTopLeftPixelCoord2.x >= LocalPixelPosDynRes.x)
		|| (TileTopLeftPixelCoord.y <= LocalPixelPosDynRes.y && TileTopLeftPixelCoord2.y >= LocalPixelPosDynRes.y))
	{
		OutLuminanceTransmittance.rgba = float4(0.25, 0.25, 0.25, 0.0);
	}
}

#endif // SHADER_DEBUG_LIGHT_GRID_PS

#ifdef FeedbackStatusCS

StructuredBuffer<uint> CulledLightDataAllocatorBuffer;
uint NumCulledLightDataEntries;

StructuredBuffer<uint> CulledLightLinkAllocatorBuffer;

uint StatusMessageId;

[numthreads(1, 1, 1)]
void FeedbackStatusCS()
{												
	uint NumAllocatedEntries = CulledLightDataAllocatorBuffer[0];
	uint NumAllocatedLinks = CulledLightLinkAllocatorBuffer[0];

	FGPUMessageWriter Mw = GPUMessageBegin(StatusMessageId, 4U);

	// max of allocated entries or links because when link allocation fail entries are not allocated
	// however every link allocation would result in an entry allocation
	GPUMessageWriteItem(Mw, max(NumAllocatedEntries, NumAllocatedLinks));
	GPUMessageWriteItem(Mw, NumCulledLightDataEntries);

	GPUMessageWriteItem(Mw, NumAllocatedLinks);
	GPUMessageWriteItem(Mw, NumAvailableLinks);
}

#endif
