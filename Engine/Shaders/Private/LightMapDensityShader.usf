// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMapDensityShader.hlsl: Shader for rendering lightmap density as a color
=============================================================================*/

#define NEEDS_LIGHTMAP_COORDINATE					(HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP)

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "/Engine/Private/BasePassCommonFunctions.ush"

struct FLightMapDensityVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 WorldPosition	: TEXCOORD6;
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO : TEXCOORD7;
#endif
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
};

#define FLightMapDensityVSOutput FLightMapDensityVSToPS

/*=============================================================================
	Vertex Shader
=============================================================================*/
#if VERTEXSHADER
void MainVertexShader(
	FVertexFactoryInput Input,
	out FLightMapDensityVSOutput Output
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 WorldPositionExcludingWPO = WorldPosition;
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);

	Output.WorldPosition = WorldPosition;
	{
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, Output.WorldPosition);
		float4 ClipSpacePosition = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);
		Output.Position = INVARIANT(ClipSpacePosition);
	}

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	Output.PixelPositionExcludingWPO = WorldPositionExcludingWPO.xyz;
#endif
}
#endif

/*=============================================================================
	Pixel Shader
=============================================================================*/

float2		LightMapResolutionScale;

/**
 *	Tagging built vs. unbuilt lighting on objects.
 *	x = 1.0, y = 0.0 for built lighting
 *	x = 0.0, y = 1.0 for unbuilt lighting
 *	z = 1.0 if the object is selected
 */
float3		BuiltLightingAndSelectedFlags;

/** 
 *	X = scalar to multiply density against...
 *  Y = scalar to multiply calculatated color against...
 *      So, to enable greyscale density mode - set X != 0, Y = 0
 *		Standard color mode - set X = 0, Y = 1
 *	Z = texture lightmap scalar
 *		Set to 1.0 if texture mapped to leave texture mapped colors alone.
 *		Set to 0.0 if vertex mapped
 *	W = vertex lightmap scalar
 *		Set to 0.0 if texture mapped
 *		Set to 1.0 if vertex mapped
 */
float4		LightMapDensityDisplayOptions;

/**
 *	The 'main' for the PixelShader
 */
void MainPixelShader(
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants,
	float4 WorldPosition	: TEXCOORD6,
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO : TEXCOORD7,
#endif
	in float4 SvPosition	: SV_Position
	OPTIONAL_IsFrontFace,
	out float4 OutColor		: SV_Target0
	)
{
	ResolvedView = ResolveView();

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);

	float2 LightMapUV,LightMapUV1;
#if HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP
	uint LightmapDataIndex;
	GetLightMapCoordinates(FactoryInterpolants,LightMapUV,LightMapUV1,LightmapDataIndex);
#else
	LightMapUV = LightMapUV1 = float2(0.1,0.1);
#endif

	// Area of parallelogram, in world space units.
	float WorldSpaceArea = length( cross( ddx(WorldPosition.xyz), ddy(WorldPosition.xyz) ) );
	WorldSpaceArea = max( WorldSpaceArea, 0.00000001f );

	float MinDensity = LightmapDensityPass.LightMapDensity.y;
	float IdealDensity = LightmapDensityPass.LightMapDensity.z;
	float MaxDensity = LightmapDensityPass.LightMapDensity.w;

	float Density = MinDensity;

	float2 TexCoord = LightMapUV * (LightMapResolutionScale.xy * 2.0);	// In texels
	float2 A = ddx(TexCoord);
	float2 B = ddy(TexCoord);
	float2 C = A.xy * B.yx;

	// Area of parallelogram, in texels.
	float TexelArea = abs( C.x - C.y );
	Density = TexelArea / WorldSpaceArea;
	// Clamp the density to the max
	Density = min( Density, MaxDensity );

	float4 TintColor;
	float4 TintGrayScale;
	if ( Density > IdealDensity )
	{
		float Range = MaxDensity - IdealDensity;
		Density -= IdealDensity;
 		TintColor = RETURN_COLOR( float4( Density/Range, (Range-Density)/Range, 0.0f, 1.0f ) );
	}
	else
	{
		float Range = IdealDensity - MinDensity;
		Density -= MinDensity;
 		TintColor = RETURN_COLOR( float4( 0.0f, Density/Range, (Range-Density)/Range, 1.0f ) )	;
	}

	float GrayScaleRange = MaxDensity - MinDensity;
	TintGrayScale = RETURN_COLOR(float4(Density/GrayScaleRange,Density/GrayScaleRange,Density/GrayScaleRange,1.0f));

	// Disable the color tinting if the option is set
	TintColor *= LightMapDensityDisplayOptions.y;
	// Enable 'grayscale' mode if desired
 	TintColor += (TintGrayScale * LightMapDensityDisplayOptions.x);
 	TintColor *= BuiltLightingAndSelectedFlags.x;
 	TintColor += BuiltLightingAndSelectedFlags.yyyy;
	// Override the coloring if vertex mapped
 	TintColor *= LightMapDensityDisplayOptions.z;
 	TintColor += (LightmapDensityPass.VertexMappedColor * LightMapDensityDisplayOptions.w);
 	// Override the coloring if selected...
 	TintColor *= (1.0f - BuiltLightingAndSelectedFlags.z);
 	TintColor += (LightmapDensityPass.DensitySelectedColor * BuiltLightingAndSelectedFlags.z);

	LightMapUV *= LightMapResolutionScale.xy;

	OutColor = Texture2DSample(LightmapDensityPass.GridTexture, LightmapDensityPass.GridTextureSampler,LightMapUV) * TintColor;
}
