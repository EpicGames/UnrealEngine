// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LightData.ush"

/** Returns the full width and height of the rect light */
float2 GetRectLightDimensions(FLightShaderParameters In)
{
	return 2.0 * float2(In.SourceRadius, In.SourceLength);
}

/** Returns the tangents to use respectively for GetRectLightDimensions() X and Y. */
void GetRectLightTangents(FLightShaderParameters In, out float3 Tangent, out float3 BiTangent)
{
	Tangent = cross(In.Tangent, In.Direction);
	BiTangent = In.Tangent;
}

// Parameters feed through FLightShaderParameters used directly as SHADER_PARAMETER_STRUCT() in a shader's root parameter structure.
float3 Light_TranslatedWorldPosition;
float  Light_InvRadius;
float3 Light_Color;
float  Light_FalloffExponent;
float3 Light_Direction;
float3 Light_Tangent;
float2 Light_SpotAngles;
float  Light_SpecularScale;
float  Light_DiffuseScale;
float  Light_SourceRadius;
float  Light_SoftSourceRadius;
float  Light_SourceLength;
float  Light_RectLightBarnCosAngle;
float  Light_RectLightBarnLength;
float2 Light_RectLightAtlasUVOffset;
float2 Light_RectLightAtlasUVScale;
float  Light_RectLightAtlasMaxLevel;
float  Light_IESAtlasIndex;
uint   Light_LightFunctionAtlasLightIndex;
uint   Light_bAffectsTranslucentLighting;

/** Returns the FLightShaderParameters from the root shader parameters. */
FLightShaderParameters GetRootLightShaderParameters()
{
	// Hopefully one day the shader compiler will be so nice we would no longer have to do this.
	FLightShaderParameters Out;
	Out.TranslatedWorldPosition = Light_TranslatedWorldPosition;
	Out.InvRadius				= Light_InvRadius;
	Out.Color					= Light_Color;
	Out.FalloffExponent			= Light_FalloffExponent;
	Out.Direction				= Light_Direction;
	Out.Tangent					= Light_Tangent;
	Out.SpotAngles				= Light_SpotAngles;
	Out.SpecularScale			= Light_SpecularScale;
	Out.DiffuseScale			= Light_DiffuseScale;
	Out.SourceRadius			= Light_SourceRadius;
	Out.SoftSourceRadius		= Light_SoftSourceRadius;
	Out.SourceLength			= Light_SourceLength;
	Out.RectLightBarnCosAngle	= Light_RectLightBarnCosAngle;
	Out.RectLightBarnLength		= Light_RectLightBarnLength;
	Out.RectLightAtlasUVOffset	= Light_RectLightAtlasUVOffset;
	Out.RectLightAtlasUVScale	= Light_RectLightAtlasUVScale;
	Out.RectLightAtlasMaxLevel	= Light_RectLightAtlasMaxLevel;
	Out.IESAtlasIndex			= Light_IESAtlasIndex;
	Out.LightFunctionAtlasLightIndex = Light_LightFunctionAtlasLightIndex;
	Out.bAffectsTranslucentLighting = Light_bAffectsTranslucentLighting;
	return Out;
}
