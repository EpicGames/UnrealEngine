// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SceneTexturesCommon.ush"

void DownsampleLightShaftsVertexMain(
	in float2 InPosition : ATTRIBUTE0,
	in float2 InUV       : ATTRIBUTE1,
	out float2 OutTexCoord : TEXCOORD0,
	out float3 OutScreenVector : TEXCOORD1,
	out float4 OutPosition : SV_POSITION
	)
{
	DrawRectangle(float4(InPosition.xy, 0, 1), InUV, OutPosition, OutTexCoord);
	OutScreenVector = ScreenVectorFromScreenRect(float4(OutPosition.xy, 1, 0));
}

/** Origin in texture coordinates in xy. */
float2 TextureSpaceBlurOrigin;

float BloomMaxBrightness;

/** Mins in xy, Maxes in zw. */
float4 UVMinMax;

float4 AspectRatioAndInvAspectRatio;

/** 1.0f / OcclusionDepthRange in x, BloomScale in y, 1 in z, OcclusionMaskDarkness in w. */
float4 LightShaftParameters;

/** Tint in rgb, threshold in a. */
float4 BloomTintAndThreshold;

/** Result of the previous pass, rgb contains bloom color and a contains an occlusion mask. */
Texture2D SourceTexture;

#define SourceTextureSampler GlobalBilinearClampedSampler

void DownsampleLightShaftsPixelMain(
	float2 InUV : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	out float4 OutColor : SV_Target0)
{
	OutColor = 1;

#if OCCLUSION_TERM

	float SceneDepth = CalcSceneDepth(InUV);

	float2 NormalizedCoordinates = (InUV - UVMinMax.xy) / UVMinMax.zw;
	// Setup a mask that is 1 at the edges of the screen and 0 at the center
	float EdgeMask = 1.0f - NormalizedCoordinates.x * (1.0f - NormalizedCoordinates.x) * NormalizedCoordinates.y * (1.0f - NormalizedCoordinates.y) * 8.0f;
	EdgeMask = EdgeMask * EdgeMask * EdgeMask * EdgeMask;

	float InvOcclusionDepthRange = LightShaftParameters.x;
	// Filter the occlusion mask instead of the depths
	float OcclusionMask = saturate(SceneDepth * InvOcclusionDepthRange);
	// Apply the edge mask to the occlusion factor
	OutColor.x = max(OcclusionMask, EdgeMask);

#else
	float3 SceneColor = CalcSceneColor(InUV);
	float SceneDepth = CalcSceneDepth(InUV);

	float2 NormalizedCoordinates = (InUV - UVMinMax.xy) / UVMinMax.zw;
	// Setup a mask that is 1 at the edges of the screen and 0 at the center
	float EdgeMask = 1.0f - NormalizedCoordinates.x * (1.0f - NormalizedCoordinates.x) * NormalizedCoordinates.y * (1.0f - NormalizedCoordinates.y) * 8.0f;
	EdgeMask = EdgeMask * EdgeMask * EdgeMask * EdgeMask;

	// Only bloom colors according if post exposure brightness is over BloomThreshold,
	// and if brightness is greater than BloomMaxBrightness, scale it down to BloomMaxBrightness to avoid saturated artefacts.
	float Luminance = max(dot(SceneColor, half3(.3f, .59f, .11f)), 6.10352e-5);
	float AdjustedLuminance = clamp(Luminance - BloomTintAndThreshold.a, 0.0f, BloomMaxBrightness);
	float3 BloomColor = LightShaftParameters.y * SceneColor / Luminance * AdjustedLuminance * 2.0f;

	float InvOcclusionDepthRange = LightShaftParameters.x;

	// Only allow bloom from pixels whose depth are in the far half of OcclusionDepthRange
	float BloomDistanceMask = saturate((SceneDepth - .5f / InvOcclusionDepthRange) * InvOcclusionDepthRange);
	// Setup a mask that is 0 at TextureSpaceBlurOrigin and increases to 1 over distance
	float BlurOriginDistanceMask = 1.0f - saturate(length(TextureSpaceBlurOrigin.xy - InUV * AspectRatioAndInvAspectRatio.zw) * 2.0f);
	// Calculate bloom color with masks applied
	OutColor.rgb = BloomColor * BloomTintAndThreshold.rgb * BloomDistanceMask * (1.0f - EdgeMask) * BlurOriginDistanceMask * BlurOriginDistanceMask;
#endif
}

float4 RadialBlurParameters;

#ifndef NUM_SAMPLES
#define NUM_SAMPLES 1
#endif

void BlurLightShaftsMain(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float3 BlurredValues = 0;
	// Scale the UVs so that the blur will be the same pixel distance in x and y
	float2 AspectCorrectedUV = UV * AspectRatioAndInvAspectRatio.zw;
	// Increase the blur distance exponentially in each pass
	float PassScale = pow(.4f * NUM_SAMPLES, RadialBlurParameters.z);
	float2 AspectCorrectedBlurVector = (TextureSpaceBlurOrigin.xy - AspectCorrectedUV)
		// Prevent reading past the light position
		* min(RadialBlurParameters.y * PassScale, 1);

	float2 BlurVector = AspectCorrectedBlurVector * AspectRatioAndInvAspectRatio.xy;

	UNROLL
	for (int SampleIndex = 0; SampleIndex < NUM_SAMPLES; SampleIndex++)
	{
		float2 SampleUVs = (AspectCorrectedUV + AspectCorrectedBlurVector * SampleIndex / (float)NUM_SAMPLES) * AspectRatioAndInvAspectRatio.xy;
		// Needed because sometimes the source texture is larger than the part we are reading from
		float2 ClampedUVs = clamp(SampleUVs, UVMinMax.xy, UVMinMax.zw);
		float3 SampleValue = Texture2DSample(SourceTexture, SourceTextureSampler, ClampedUVs).xyz;
		BlurredValues += SampleValue;
	}
	
	OutColor.rgb = BlurredValues / (float)NUM_SAMPLES;
	OutColor.a = 1;
}

void FinishOcclusionMain(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;

	float LightShaftOcclusion = Texture2DSample(SourceTexture, SourceTextureSampler, UV).x;

	// LightShaftParameters.w is OcclusionMaskDarkness, use that to control what an occlusion value of 0 maps to
	float FinalOcclusion = lerp(LightShaftParameters.w, 1, LightShaftOcclusion * LightShaftOcclusion);
	// Setup a mask based on where the blur origin is
	float BlurOriginDistanceMask = saturate(length(TextureSpaceBlurOrigin.xy - UV * AspectRatioAndInvAspectRatio.zw) * .2f);
	// Fade out occlusion over distance away from the blur origin
	FinalOcclusion = lerp(FinalOcclusion, 1, BlurOriginDistanceMask);
	
	OutColor = float4(FinalOcclusion, 1, 1, 1);
}

void ApplyLightShaftsPixelMain(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float4 LightShaftColorAndMask = Texture2DSample(SourceTexture, SourceTextureSampler, clamp(UV, UVMinMax.xy, UVMinMax.zw));

	OutColor = RETURN_COLOR(float4(LightShaftColorAndMask.rgb, 1));
}