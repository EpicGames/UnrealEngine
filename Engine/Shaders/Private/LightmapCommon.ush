// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShadingCommon.ush"
#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"
#include "VolumetricLightmapShared.ush"

#if !NUM_VIRTUALTEXTURE_SAMPLES && !LIGHTMAP_VT_ENABLED
#define VTPageTableResult float
#endif

#if LIGHTMAP_VT_ENABLED
	#include "VirtualTextureCommon.ush"
#endif
#include "LightmapData.ush"

#if IS_NANITE_PASS
#define LightmapUVType FloatDeriv2
#else
#define LightmapUVType float2
#endif

float2 ScaleLightmapUV(float2 UV, float2 Scale)
{
	return UV * Scale;
}

FloatDeriv2 ScaleLightmapUV(FloatDeriv2 UV, float2 Scale)
{
	UV.Value *= Scale;
	UV.Ddx *= Scale;
	UV.Ddy *= Scale;
	return UV;
}

#if HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP
 
#ifndef MATERIAL_USE_LM_DIRECTIONALITY
#define MATERIAL_USE_LM_DIRECTIONALITY	1
#endif

// Material quality overrides
#ifndef QL_FORCEDISABLE_LM_DIRECTIONALITY
#define QL_FORCEDISABLE_LM_DIRECTIONALITY 0
#endif

#define USE_LM_DIRECTIONALITY (MATERIAL_USE_LM_DIRECTIONALITY && !QL_FORCEDISABLE_LM_DIRECTIONALITY)

#define LIGHTMAP_VTADDRESSMODE VTADDRESSMODE_CLAMP

#if LIGHTMAP_VT_ENABLED
VTPageTableResult LightmapGetVTSampleInfo(float2 UV, uint LightmapDataIndex, float2 SvPositionXY)
{
	UV = ScaleLightmapUV(UV, float2(1.0f, 2.0f)); // Undo transform used to pack 2 lightmap coeffs in 1 texture for the non-VT default case

	return TextureLoadVirtualPageTable(LightmapResourceCluster.LightmapVirtualTexturePageTable0, LightmapResourceCluster.LightmapVirtualTexturePageTable1,
		VTPageTableUniform_Unpack(GetLightmapData(LightmapDataIndex).LightmapVTPackedPageTableUniform[0], GetLightmapData(LightmapDataIndex).LightmapVTPackedPageTableUniform[1]),
		UV, LIGHTMAP_VTADDRESSMODE, LIGHTMAP_VTADDRESSMODE, 0, SvPositionXY);
}

VTPageTableResult LightmapGetVTSampleInfo(FloatDeriv2 UV, uint LightmapDataIndex, float2 SvPositionXY)
{
	UV = ScaleLightmapUV(UV, float2(1.0f, 2.0f)); // Undo transform used to pack 2 lightmap coeffs in 1 texture for the non-VT default case
	
	return TextureLoadVirtualPageTableGrad(LightmapResourceCluster.LightmapVirtualTexturePageTable0, LightmapResourceCluster.LightmapVirtualTexturePageTable1,
		VTPageTableUniform_Unpack(GetLightmapData(LightmapDataIndex).LightmapVTPackedPageTableUniform[0], GetLightmapData(LightmapDataIndex).LightmapVTPackedPageTableUniform[1]),
		UV.Value, LIGHTMAP_VTADDRESSMODE, LIGHTMAP_VTADDRESSMODE, UV.Ddx, UV.Ddy, SvPositionXY);
}

half4 SampleLightmapVT(VTPageTableResult LightmapVTPageTableResult, uint LayerIndex, uint LightmapDataIndex, Texture2D CacheTexture, SamplerState CacheSampler)
{
	return TextureVirtualSample(
		CacheTexture, CacheSampler,
		LightmapVTPageTableResult, LayerIndex,
		VTUniform_Unpack(GetLightmapData(LightmapDataIndex).LightmapVTPackedUniform[LayerIndex]));
}
#endif // LIGHTMAP_VT_ENABLED

void GetLightMapColorLQ( VTPageTableResult LightmapVTPageTableResult, LightmapUVType LightmapUV0, LightmapUVType LightmapUV1, uint LightmapDataIndex, half3 WorldNormal, bool bEvaluateBackface, out half3 OutDiffuseLighting, out half3 OutSubsurfaceLighting)
{
	OutSubsurfaceLighting = 0;

#if LIGHTMAP_VT_ENABLED
	half4 Lightmap0 = SampleLightmapVT(LightmapVTPageTableResult, 0u, LightmapDataIndex, LightmapResourceCluster.VTLightMapTexture, LightmapResourceCluster.LightMapSampler);
	half4 Lightmap1 = SampleLightmapVT(LightmapVTPageTableResult, 1u, LightmapDataIndex, LightmapResourceCluster.VTLightMapTexture_1, LightmapResourceCluster.LightMapSampler_1);
#else
	half4 Lightmap0 = Texture2DSample( LightmapResourceCluster.LightMapTexture, LightmapResourceCluster.LightMapSampler, LightmapUV0 );
	half4 Lightmap1 = Texture2DSample( LightmapResourceCluster.LightMapTexture, LightmapResourceCluster.LightMapSampler, LightmapUV1 );
#endif
		
	// Range scale
	half3 LogRGB = Lightmap0.rgb * GetLightmapData(LightmapDataIndex).LightMapScale[0].xyz + GetLightmapData(LightmapDataIndex).LightMapAdd[0].xyz;	// 1 vmad

	half LogL = Luminance( LogRGB );					// 1 dot

	// LogL -> L
	const half LogBlackPoint = 0.00390625;	// exp2(-8);
	half L = exp2( LogL * 16 - 8 ) - LogBlackPoint;		// 1 exp2, 1 smad, 1 ssub

#if USE_LM_DIRECTIONALITY
	// Alpha doesn't matter, will scaled by zero
	float4 SH = Lightmap1 * GetLightmapData(LightmapDataIndex).LightMapScale[1] + GetLightmapData(LightmapDataIndex).LightMapAdd[1];	// 1 vmad

	// Sample SH with normal
	half Directionality = max( 0.0, dot( SH, float4(WorldNormal.yzx, 1) ) );	// 1 dot, 1 smax

	#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
	if (bEvaluateBackface)
	{
		half SubsurfaceDirectionality = max(0.0, dot(SH, float4(-WorldNormal.yzx, 1)));
		half SubsurfaceLuma = L * SubsurfaceDirectionality;
		OutSubsurfaceLighting = LogRGB * (SubsurfaceLuma / LogL);
	}
	#endif
#else
	half Directionality = 0.6;

	#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
	if (bEvaluateBackface)
	{
		half SubsurfaceLuma = L * Directionality;
		OutSubsurfaceLighting = LogRGB * (SubsurfaceLuma / LogL);
	}
	#endif
#endif
		
	half Luma = L * Directionality;
	half3 Color = LogRGB * (Luma / max(0.00001f, LogL));	// 1 smax, 1 rcp, 1 smul, 1 vmul

	OutDiffuseLighting = Color;
}

void GetLightMapColorHQ( VTPageTableResult LightmapVTPageTableResult, LightmapUVType LightmapUV0, LightmapUVType LightmapUV1, uint LightmapDataIndex, half3 WorldNormal, float2 SvPositionXY, bool bEvaluateBackface, out half3 OutDiffuseLighting, out half3 OutSubsurfaceLighting )
{
	OutSubsurfaceLighting = 0;

	half4 Lightmap0;
	half4 Lightmap1;
#if LIGHTMAP_VT_ENABLED
	Lightmap0 = SampleLightmapVT( LightmapVTPageTableResult, 0u, LightmapDataIndex, LightmapResourceCluster.VTLightMapTexture, LightmapResourceCluster.LightMapSampler); 
	Lightmap1 = SampleLightmapVT( LightmapVTPageTableResult, 1u, LightmapDataIndex, LightmapResourceCluster.VTLightMapTexture_1, LightmapResourceCluster.LightMapSampler); 
#else
	Lightmap0 = Texture2DSample( LightmapResourceCluster.LightMapTexture, LightmapResourceCluster.LightMapSampler, LightmapUV0 );
	Lightmap1 = Texture2DSample( LightmapResourceCluster.LightMapTexture, LightmapResourceCluster.LightMapSampler, LightmapUV1 );
#endif

	half LogL = Lightmap0.w;

	// Add residual
	LogL += Lightmap1.w * (1.0 / 255) - (0.5 / 255);

	// Range scale LogL
	LogL = LogL * GetLightmapData(LightmapDataIndex).LightMapScale[0].w + GetLightmapData(LightmapDataIndex).LightMapAdd[0].w;
		
	// Range scale UVW
	half3 UVW = Lightmap0.rgb * Lightmap0.rgb * GetLightmapData(LightmapDataIndex).LightMapScale[0].rgb + GetLightmapData(LightmapDataIndex).LightMapAdd[0].rgb;

	// LogL -> L
	const half LogBlackPoint = 0.01858136;
	half L = exp2( LogL ) - LogBlackPoint;

#if USE_LM_DIRECTIONALITY
	// Range scale SH. Alpha doesn't matter, will scale with zero
	float4 SH = Lightmap1 * GetLightmapData(LightmapDataIndex).LightMapScale[1] + GetLightmapData(LightmapDataIndex).LightMapAdd[1];

	// Sample SH with normal
	half Directionality = max( 0.0, dot( SH, float4(WorldNormal.yzx, 1) ) );

	#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
	if (bEvaluateBackface)
	{
		half SubsurfaceDirectionality = max(0.0, dot(SH, float4(-WorldNormal.yzx, 1)));
		OutSubsurfaceLighting = L * SubsurfaceDirectionality * UVW;
	}
	#endif
#else
	half Directionality = 0.6;

	#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
	if (bEvaluateBackface)
	{
		OutSubsurfaceLighting = L * Directionality * UVW;
	}
	#endif
#endif

	half Luma = L * Directionality;
	half3 Color = Luma * UVW;

	OutDiffuseLighting = Color;
}

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define LIGHTMAP_SHARED_SAMPLER( Sampler ) LightmapResourceCluster.LightMapSampler
#else
#define LIGHTMAP_SHARED_SAMPLER( Sampler ) LightmapResourceCluster.Sampler
#endif

float4 GetSkyBentNormalAndOcclusion(VTPageTableResult LightmapVTPageTableResult, LightmapUVType LightmapUV, uint LightmapDataIndex, float2 SvPositionXY)
{
	float4 TextureValue;
#if LIGHTMAP_VT_ENABLED
	TextureValue = SampleLightmapVT( LightmapVTPageTableResult, 3u, LightmapDataIndex, LightmapResourceCluster.VTSkyOcclusionTexture, LIGHTMAP_SHARED_SAMPLER(SkyOcclusionSampler)); 
#else
	TextureValue = Texture2DSample( LightmapResourceCluster.SkyOcclusionTexture, LIGHTMAP_SHARED_SAMPLER(SkyOcclusionSampler), LightmapUV );
#endif

	// Unpack vector
	TextureValue.rgb = TextureValue.rgb * 2 - 1;
	// Undo sqrt which allocated more precision toward 0
	TextureValue.a = TextureValue.a * TextureValue.a;
	return TextureValue;
}

float GetAOMaterialMask(VTPageTableResult LightmapVTPageTableResult, LightmapUVType LightmapUV, uint LightmapDataIndex, float2 SvPositionXY)
{
	float TextureValue;
#if LIGHTMAP_VT_ENABLED
	TextureValue = SampleLightmapVT( LightmapVTPageTableResult, 4u, LightmapDataIndex, LightmapResourceCluster.VTAOMaterialMaskTexture, LIGHTMAP_SHARED_SAMPLER(AOMaterialMaskSampler)).x; 
#else
	TextureValue = Texture2DSample( LightmapResourceCluster.AOMaterialMaskTexture, LIGHTMAP_SHARED_SAMPLER(AOMaterialMaskSampler), LightmapUV ).x;
#endif

	// Undo sqrt which allocated more precision toward 0
	return TextureValue * TextureValue;
}

#endif

// Used by deferred renderer only
half4 GetPrecomputedShadowMasks(VTPageTableResult LightmapVTPageTableResult, FVertexFactoryInterpolantsVSToPS Interpolants, FMaterialPixelParameters MaterialParameters, float3 VolumetricLightmapBrickTextureUVs)
{
	// Note: WRITES_PRECSHADOWFACTOR_ZERO have to match the logic here
	#if STATICLIGHTING_TEXTUREMASK && STATICLIGHTING_SIGNEDDISTANCEFIELD

		// VT doesn't need ShadowMapCoordinate (instead shared lightmap UV), but still needs LightmapDataIndex
		LightmapUVType ShadowMapCoordinate;
		uint LightmapDataIndex;
		GetShadowMapCoordinate(Interpolants, ShadowMapCoordinate, LightmapDataIndex);

		// Fetch the 4 channels of distance field data
		half4 DistanceField;
		#if LIGHTMAP_VT_ENABLED
			DistanceField = SampleLightmapVT( LightmapVTPageTableResult, 2u, LightmapDataIndex, LightmapResourceCluster.VTStaticShadowTexture, LIGHTMAP_SHARED_SAMPLER(StaticShadowTextureSampler));
		#else
			DistanceField = Texture2DSample(LightmapResourceCluster.StaticShadowTexture, LIGHTMAP_SHARED_SAMPLER(StaticShadowTextureSampler), ShadowMapCoordinate);
		#endif

		float4 InvUniformPenumbraSizes = GetLightmapData(LightmapDataIndex).InvUniformPenumbraSizes;
		float4 DistanceFieldBias = -.5f * InvUniformPenumbraSizes + .5f;

		// Compute shadow factors by scaling and biasing the distance
		half4 ShadowFactors = saturate(DistanceField * InvUniformPenumbraSizes + DistanceFieldBias);
		return GetLightmapData(LightmapDataIndex).StaticShadowMapMasks * ShadowFactors * ShadowFactors;

	#elif HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP

		// Mark as shadowed for lightmapped objects with no shadowmap
		// This is necessary because objects inside a light's influence that were determined to be completely shadowed won't be rendered with STATICLIGHTING_TEXTUREMASK==1
		return 0;

	#elif ALLOW_STATIC_LIGHTING

		float DirectionalLightShadowing = 1.0f;

	#if CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING
		if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_USE_SINGLE_SAMPLE_SHADOW_SL) != 0 && View.IndirectLightingCacheShowFlag > 0.0f)
		{
			DirectionalLightShadowing = IndirectLightingCache.DirectionalLightShadowing;
		}
	#endif

		BRANCH
		if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_USE_VOLUMETRIC_LM_SHADOW_SL) != 0)
		{
			#if !PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
				// Compute brick UVs if we haven't already
				VolumetricLightmapBrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WSHackToFloat(GetWorldPosition(MaterialParameters)));
			#endif

			DirectionalLightShadowing = GetVolumetricLightmapDirectionalLightShadowing(VolumetricLightmapBrickTextureUVs);
		}

		// Directional light is always packed into the first static shadowmap channel, so output the per-primitive directional light shadowing there if requested
		return half4(DirectionalLightShadowing, 1, 1, 1);

	#else

		return half4(1, 1, 1, 1);

	#endif
}

// Used by mobile renderer only
half4 GetPrimaryPrecomputedShadowMask(VTPageTableResult LightmapVTPageTableResult, FVertexFactoryInterpolantsVSToPS Interpolants, FMaterialPixelParameters MaterialParameters)
{
	#if STATICLIGHTING_TEXTUREMASK && STATICLIGHTING_SIGNEDDISTANCEFIELD

		LightmapUVType ShadowMapCoordinate;
		uint LightmapDataIndex;
		GetShadowMapCoordinate(Interpolants, ShadowMapCoordinate, LightmapDataIndex);

		// Fetch the distance field data
		#if LIGHTMAP_VT_ENABLED
			half4 DistanceField = SampleLightmapVT(LightmapVTPageTableResult, 2u, LightmapDataIndex, LightmapResourceCluster.VTStaticShadowTexture, LightmapResourceCluster.StaticShadowTextureSampler);
		#else
			half4 DistanceField = Texture2DSample(LightmapResourceCluster.StaticShadowTexture, LightmapResourceCluster.StaticShadowTextureSampler, ShadowMapCoordinate);
		#endif
		float4 InvUniformPenumbraSizes = GetLightmapData(LightmapDataIndex).InvUniformPenumbraSizes;
		float4 DistanceFieldBias = -.5f * InvUniformPenumbraSizes + .5f;
		// Compute shadow factors by scaling and biasing the distance
		half4 ShadowFactor = saturate( DistanceField * InvUniformPenumbraSizes + DistanceFieldBias );
		return GetLightmapData(LightmapDataIndex).StaticShadowMapMasks * ShadowFactor * ShadowFactor;
	#elif HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP
		// Mark as shadowed for lightmapped objects with no shadowmap
		// This is necessary because objects inside a light's influence that were determined to be completely shadowed won't be rendered with STATICLIGHTING_TEXTUREMASK==1
		return 0.0f;
	#else
		#if CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING
			// output per-primitive directional light shadowing if requested
			if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_USE_SINGLE_SAMPLE_SHADOW_SL) != 0 && ResolvedView.IndirectLightingCacheShowFlag > 0.0f)
			{
				return half4(IndirectLightingCache.DirectionalLightShadowing, 1, 1, 1);
			}
		#endif
	#endif

	return 1.0f;
}
