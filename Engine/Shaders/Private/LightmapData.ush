// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * LightmapData.ush
 */

#pragma once

#ifndef USE_GLOBAL_GPU_LIGHTMAP_DATA
	#define USE_GLOBAL_GPU_LIGHTMAP_DATA 0
#endif

 // Must match FPrecomputedLightingUniformParameters in C++
struct FLightmapSceneData
{
	float4 StaticShadowMapMasks;
	float4 InvUniformPenumbraSizes;
	float4 LightMapCoordinateScaleBias;
	float4 ShadowMapCoordinateScaleBias;
	float4 LightMapScale[2];
	float4 LightMapAdd[2];
	uint4 LightmapVTPackedPageTableUniform[2];
	uint4 LightmapVTPackedUniform[5];
};

FLightmapSceneData GetLightmapDataFromUniformBuffer()
{
	FLightmapSceneData LightmapData;
	LightmapData.StaticShadowMapMasks = PrecomputedLightingBuffer.StaticShadowMapMasks;
	LightmapData.InvUniformPenumbraSizes = PrecomputedLightingBuffer.InvUniformPenumbraSizes;
	LightmapData.LightMapCoordinateScaleBias = PrecomputedLightingBuffer.LightMapCoordinateScaleBias;
	LightmapData.ShadowMapCoordinateScaleBias = PrecomputedLightingBuffer.ShadowMapCoordinateScaleBias;
	LightmapData.LightMapScale[0] = PrecomputedLightingBuffer.LightMapScale[0];
	LightmapData.LightMapScale[1] = PrecomputedLightingBuffer.LightMapScale[1];
	LightmapData.LightMapAdd[0] = PrecomputedLightingBuffer.LightMapAdd[0];
	LightmapData.LightMapAdd[1] = PrecomputedLightingBuffer.LightMapAdd[1];
	LightmapData.LightmapVTPackedPageTableUniform[0] = PrecomputedLightingBuffer.LightmapVTPackedPageTableUniform[0];
	LightmapData.LightmapVTPackedPageTableUniform[1] = PrecomputedLightingBuffer.LightmapVTPackedPageTableUniform[1];
	UNROLL
	for (uint i = 0u; i < 5u; ++i)
	{
		LightmapData.LightmapVTPackedUniform[i] = PrecomputedLightingBuffer.LightmapVTPackedUniform[i];
	}
	return LightmapData;
}

#if VF_USE_PRIMITIVE_SCENE_DATA

// Stride of a single lightmap data entry in float4's, must match C++
#define LIGHTMAP_SCENE_DATA_STRIDE 15

#if USE_GLOBAL_GPU_LIGHTMAP_DATA
StructuredBuffer<float4> GPUSceneLightmapData;
#endif

float4 LoadLightmapDataElement(uint Index)
{
#if USE_GLOBAL_GPU_LIGHTMAP_DATA
	checkStructuredBufferAccessSlow(GPUSceneLightmapData, Index);
	return GPUSceneLightmapData[Index];
#else
	checkStructuredBufferAccessSlow(Scene.GPUScene.GPUSceneLightmapData, Index);
	return Scene.GPUScene.GPUSceneLightmapData[Index];
#endif
}

// Fetch from scene lightmap data buffer
FLightmapSceneData GetLightmapData(uint LightmapDataIndex) 
{
	FLightmapSceneData LightmapData;

	// Note: layout must match FLightmapSceneShaderData in C++
	// Relying on optimizer to remove unused loads
	uint LightmapDataBaseOffset = LightmapDataIndex * LIGHTMAP_SCENE_DATA_STRIDE;
	LightmapData.StaticShadowMapMasks = LoadLightmapDataElement(LightmapDataBaseOffset + 0);
	LightmapData.InvUniformPenumbraSizes = LoadLightmapDataElement(LightmapDataBaseOffset + 1);
	LightmapData.LightMapCoordinateScaleBias = LoadLightmapDataElement(LightmapDataBaseOffset + 2);
	LightmapData.ShadowMapCoordinateScaleBias = LoadLightmapDataElement(LightmapDataBaseOffset + 3);
	LightmapData.LightMapScale[0] = LoadLightmapDataElement(LightmapDataBaseOffset + 4);
	LightmapData.LightMapScale[1] = LoadLightmapDataElement(LightmapDataBaseOffset + 5);
	LightmapData.LightMapAdd[0] = LoadLightmapDataElement(LightmapDataBaseOffset + 6);
	LightmapData.LightMapAdd[1] = LoadLightmapDataElement(LightmapDataBaseOffset + 7);
	LightmapData.LightmapVTPackedPageTableUniform[0] = asuint(LoadLightmapDataElement(LightmapDataBaseOffset + 8));
	LightmapData.LightmapVTPackedPageTableUniform[1] = asuint(LoadLightmapDataElement(LightmapDataBaseOffset + 9));
	UNROLL
	for (uint i = 0u; i < 5u; ++i)
	{
		LightmapData.LightmapVTPackedUniform[i] = asuint(LoadLightmapDataElement(LightmapDataBaseOffset + 10 + i));
	}

	return LightmapData;
}

FLightmapSceneData GetLightmapData(uint LightmapDataIndex, uint DrawInstanceId)
{
#if SHADER_USES_PRIMITIVE_UBO
	#if USE_INSTANCE_CULLING
		return GetLightmapDataFromUniformBuffer();
	#else
		FLightmapSceneData LightmapData = (FLightmapSceneData)0;
		LightmapData.LightMapCoordinateScaleBias = LoadPrimitiveDataElementUBO(DrawInstanceId, 0u);
		LightmapData.ShadowMapCoordinateScaleBias = LoadPrimitiveDataElementUBO(DrawInstanceId, 1u);
		return LightmapData;
	#endif
#else	
	return GetLightmapData(LightmapDataIndex);
#endif
}

#else 

// Route to uniform buffer
FLightmapSceneData GetLightmapData(uint LightmapDataIndex)
{
	return GetLightmapDataFromUniformBuffer();
}

#endif