// Copyright Epic Games, Inc. All Rights Reserved.
#include "../Common.ush"

#include "../SHCommon.ush"
#include "../ParticipatingMediaCommon.ush"
#include "../ReflectionEnvironmentShared.ush"
#include "../FastMath.ush"



///////////////////////////////////////////////////////////////////////////////
// Local Fog Volume Data

struct FLocalFogVolumeGPUInstanceData
{
	float3x3	PreTranslatedInvTransform;
	float3		TranslatedWorlPos;
	float		UniformScale;
	float		UniformScaleInv;		// Derived from UniformScale
	float		StartDistance;

	float		RadialFogExtinction;
	float		HeightFogExtinction;
	float		HeightFogFalloff;
	float		HeightFogOffset;

	float3		Emissive;
	float3		Albedo;
	float		PhaseG;
};


// Structured buffers cannot be sampled from the vertex shader on some mobile platforms. 
// That is why simple Buffers are used for LocalFogVolumeInstances, because this tech needs to run on mobile platforms.

#if defined(LFVStruct)

#define LFVInsts						LFVStruct.LFV.LocalFogVolumeInstances
#define LFVTileDataTex					LFVStruct.LFV.LocalFogVolumeTileDataTexture

#define LFVTileDataResolution			LFVStruct.LFV.LocalFogVolumeTileDataTextureResolution
#define LFVInstcount					LFVStruct.LFV.LocalFogVolumeInstanceCount
#define LFVTilePixelSize				LFVStruct.LFV.LocalFogVolumeTilePixelSize
#define LFVMaxDensityIntoVolumetricFog	LFVStruct.LFV.LocalFogVolumeMaxDensityIntoVolumetricFog
#define LFVRenderInVolumetricFog		LFVStruct.LFV.ShouldRenderLocalFogVolumeInVolumetricFog
#define LFVGlobalStartDistance			LFVStruct.LFV.GlobalStartDistance
#define LFVHalfResTextureSizeAndInvSize	LFVStruct.LFV.HalfResTextureSizeAndInvSize

#define LFVDirectionalLightColor		LFVStruct.LFV.DirectionalLightColor
#define LFVDirectionalLightDirection	LFVStruct.LFV.DirectionalLightDirection

#else

uint2									LFV_LocalFogVolumeTileDataTextureResolution;
uint									LFV_LocalFogVolumeInstanceCount;
uint									LFV_LocalFogVolumeTilePixelSize;
float									LFV_LocalFogVolumeMaxDensityIntoVolumetricFog;
uint									LFV_ShouldRenderLocalFogVolumeInVolumetricFog;
float									LFV_GlobalStartDistance;
float4									LFV_HalfResTextureSizeAndInvSize;

float3									LFV_DirectionalLightColor;
float3									LFV_DirectionalLightDirection;

Buffer<float4>							LFV_LocalFogVolumeInstances;
Texture2DArray<uint>					LFV_LocalFogVolumeTileDataTexture;

#define LFVInsts						LFV_LocalFogVolumeInstances
#define LFVTileDataTex					LFV_LocalFogVolumeTileDataTexture

#define LFVTileDataResolution			LFV_LocalFogVolumeTileDataTextureResolution
#define LFVInstcount					LFV_LocalFogVolumeInstanceCount
#define LFVTilePixelSize				LFV_LocalFogVolumeTilePixelSize
#define LFVMaxDensityIntoVolumetricFog	LFV_LocalFogVolumeMaxDensityIntoVolumetricFog
#define LFVRenderInVolumetricFog		LFV_ShouldRenderLocalFogVolumeInVolumetricFog
#define LFVGlobalStartDistance			LFV_GlobalStartDistance
#define LFVHalfResTextureSizeAndInvSize	LFV_HalfResTextureSizeAndInvSize

#define LFVDirectionalLightColor		LFV_DirectionalLightColor
#define LFVDirectionalLightDirection	LFV_DirectionalLightDirection

#endif


FLocalFogVolumeGPUInstanceData GetLocalFogVolumeGPUInstanceData(uint Index)
{
	FLocalFogVolumeGPUInstanceData Data;
	uint Offset = Index * 3;

	float4 Data0 = LFVInsts[Offset + 0];
	float4 Data1 = LFVInsts[Offset + 1];
	float4 Data2 = LFVInsts[Offset + 2];
	// See GetLocalFogVolumeViewSortingData for the packing mathod.

	// Data0
	Data.TranslatedWorlPos	= Data0.xyz;
	Data.UniformScale		= Data0.w;
	Data.UniformScaleInv	= 1.0f / Data.UniformScale;

	// Data1
	float3 XVec;
	float3 YVec;
	XVec.xy		= UnpackFloat2FromUInt(asuint(Data1.x));
	float2 Temp	= UnpackFloat2FromUInt(asuint(Data1.y));
	XVec.z		= Temp.x;
	YVec.x		= Temp.y;
	YVec.yz		= UnpackFloat2FromUInt(asuint(Data1.z));

	float3 ZVec = cross(XVec, YVec);

	Data.PreTranslatedInvTransform = float3x3(float3(XVec * Data.UniformScaleInv), float3(YVec * Data.UniformScaleInv), float3(ZVec * Data.UniformScaleInv));
	Data.StartDistance		= Data1.w;		// Could be 2xfp16

	// Data2
	float3 Data2X			= UnpackR11G11B10F(asuint(Data2.x));
	Data.RadialFogExtinction= Data2X.x;
	Data.HeightFogExtinction= Data2X.y;
	Data.HeightFogFalloff	= Data2X.z;

	uint PackEmissiveData   = asuint(Data2.y);
	if (PackEmissiveData > 0)
	{
		Data.Emissive = UnpackR11G11B10F(PackEmissiveData);
	}
	else
	{
		Data.Emissive = 0.0f.xxx;
	}

	float4 Data2Z			= UnpackRGBA8(asuint(Data2.z));
	Data.Albedo				= Data2Z.rgb;
	Data.PhaseG				= Data2Z.a;
	Data.HeightFogOffset	= Data2.w;		// Could be 2xfp16

	return Data;
}



///////////////////////////////////////////////////////////////////////////////
// Local Fog Volume Tile Data

uint2 LFVUnpackTile(uint In)
{
	return uint2(In & 0xFFFF, In >> 16);
}

uint LFVPackTile(uint2 TileCoord)
{
#if COMPILER_SUPPORTS_PACK_INTRINSICS
	return PackUInt2ToUInt(TileCoord.x, TileCoord.y);
#else
	return TileCoord.x | (TileCoord.y << 16); // assumes 16bit is enough to represent a tiled resolution up to 65,535 :)
#endif
}



///////////////////////////////////////////////////////////////////////////////
// Local Fog Volume Integrals

struct FFogData
{
	float IntegratedLuminanceFactor;
	float Coverage;
};

FFogData LocalFogVolumeEvaluateAnalyticalIntegral(FLocalFogVolumeGPUInstanceData FogInstance, float3 RayStartU, float3 RayDirU, float RayLengthU)
{
	FFogData FogData;

	float RadialOpticalDepth = 0.0f;
	float HeightOpticalDepth = 0.0f;

	if (FogInstance.RadialFogExtinction > 0.0f)
	{
		//////////
		// Radial fog is evaluated according to spatially varying density defined as "Density * (1 - (d*d)/(r*r))" in the unint sphere space, where density is in fact Extinction.
		// An explanation of it can be found in https://iquilezles.org/articles/spheredensity/

		float3 SphereCenter = 0.0f;
		float3 VolumeCenterToRayO = (RayStartU - SphereCenter);

		float b = dot(RayDirU, VolumeCenterToRayO);
		float c = dot(VolumeCenterToRayO, VolumeCenterToRayO) - 1.0f;
		float h = b * b - c;

		if (h >= 0.0)
		{
			h = sqrt(h);
			float Length0 = -b - h;
			float Length1 = -b + h;

			Length0 = max(Length0, 0.0);
			Length1 = max(Length1, 0.0);
			Length1 = min(Length1, RayLengthU);

			// Integral of a Extinction that descreases according to square distance from the center of the volume.
			const float Length0Sqr = Length0 * Length0;
			const float Length1Sqr = Length1 * Length1;
			float Integral0 = -(c * Length0 + b * Length0Sqr + Length0Sqr * Length0 / 3.0f);
			float Integral1 = -(c * Length1 + b * Length1Sqr + Length1Sqr * Length1 / 3.0f);
			RadialOpticalDepth = max(0.0, FogInstance.RadialFogExtinction * (Integral1 - Integral0) * (3.0f / 4.0f));
		}
	}

	if (FogInstance.HeightFogExtinction > 0.0f)
	{
		//////////
		// Height fog is evaluated according to spatially varying density defined as "Density * Exp[-HeightFogFalloff * z]", where density is in fact Extinction.
		// An explanation of it can be found in https://iquilezles.org/articles/fog/.

		float StartHeight = RayStartU.z - FogInstance.HeightFogOffset; // account for the fog offset.

		// We are going to divide by RayDirU.z so we need it to be never be 0.
		const float SafeDirThreshold = 0.0001;
		const float SignRayDirUZ = RayDirU.z >= 0.0f ? 1.0f : -1.0f;
		const float SafeRayDirUZ = (abs(RayDirU.z) < SafeDirThreshold) ? SafeDirThreshold * SignRayDirUZ : RayDirU.z;

	#if 0
		// Original integral, less artefact
		HeightOpticalDepth = (FogInstance.HeightFogExtinction / FogInstance.HeightFogFalloff) * exp(-StartHeight * FogInstance.HeightFogFalloff) * (1.0 - exp(-SafeRayDirUZ * RayLengthU * FogInstance.HeightFogFalloff)) / SafeRayDirUZ;
	#elif 1
		// Reworked to avoid large value as input to exp
		//OpticalDepth = (FogInstance.HeightFogExtinction / FogInstance.HeightFogFalloff) * (exp(-StartHeight * FogInstance.HeightFogFalloff) - exp(-StartHeight * FogInstance.HeightFogFalloff - RayDir.z * RayLength * FogInstance.HeightFogFalloff)) / RayDir.z;

		// But we use an updated formulation to avoid large input to exp leading to inf and nan.
		float Factor0 = StartHeight * FogInstance.HeightFogFalloff;
		float Factor1 = SafeRayDirUZ * RayLengthU * FogInstance.HeightFogFalloff;
		Factor0 = max(-80.0f, Factor0); // Simply clamp Factor0 to a valid range in order to not have exp explode.

		HeightOpticalDepth += (FogInstance.HeightFogExtinction / (FogInstance.HeightFogFalloff * SafeRayDirUZ)) * (exp(-Factor0) - exp(-(Factor0 + Factor1)));
	#endif

		/*
		Proof of the height fog integral equations using Mathematica
		This show that IntegratedLuminanceFactor is really equivalent to coverage in this simple case.

		(*Density of matter according to height Y*)
		Density [y_] := A * Exp[-B * y]

		(*Ray position*)
		Ray[t_] := Oz + t * Rz

		(*Integrate Optical Depth for a distance T*)
		Fog[T_] := Integrate[Density[Ray[t]], {t, 0, T}]

		(*Optical Depth integration equation*)
		Fog[T]

		(*Transmittance from Optical Depth*)
		Exp[-Integrate[Density[Ray[t]], {t, 0, T}]]

		(*Transmittance from a position to the origin of tracing*)
		Trans[T_] := Exp[-Integrate[Density[Ray[t]], {t, 0, T}]]

		(*Integrate emissive and scattering from height fog, assuming density \
		is extinction and extinction=scattering, thus albedo=1*)
		Integrate[Density[Ray[t]]*EmSc *Trans[t], {t, 0, T}]

		=> this leads to EmSc - Transmittance * EmSc
						 EmSc * (1 - Transmittance)
						 EmSc * Coverage
		*/
	}

#if 1 
	// => Reference path
	// We now combine the two optical depth for the "opacity"=1-transmittance to be combined together.
	// This helps to have soft edges when using height fog. We also account for that when voxelising in the volumetric fog.
	float TR = exp(-RadialOpticalDepth);
	float TH = exp(-HeightOpticalDepth);
	float T = 1 - (1 - TR) * (1 - TH);
	float OpticalDepth = -log(T);

	// Evaluate transmittance, accounting for the volume scale converted to local unit space meter.
	float Transmittance = exp(-OpticalDepth * FogInstance.UniformScale * CENTIMETER_TO_METER);
#else 
	// => Optimised path, but does not match due to CommonFactor not being applied at the same place
	const float CommonFactor = FogInstance.UniformScale * CENTIMETER_TO_METER;
	float TR = exp(-RadialOpticalDepth);
	float TH = exp(-HeightOpticalDepth);
	float Transmittance = 1 - (1 - TR) * (1 - TH);
#endif

	// Assuming extinction is grey scale, we can compute a single coverage value from transmittance
	FogData.Coverage = (1.0 - Transmittance);
	// Assuming extinction==scattering, i.e. albedo=1, we can compute the integrale for each point on a ray while accouting for transmittance back to the view.
	// It turns out that, under these conditions, the luminance and emissive factor is simply coverage. => see Mathematica details at the bottom of this page.
	FogData.IntegratedLuminanceFactor = FogData.Coverage;

	return FogData;
}



///////////////////////////////////////////////////////////////////////////////
// Local Fog Volume Evaluation

float3 LocalFogVolumeEvaluateInScattering(in FLocalFogVolumeGPUInstanceData FogInstance, in FFogData FogData, in float3 RayDirWorld)
{
	half3 InScattering = 0.0f;

	// Lighting equation here matches the volumetric fog.

	// Main forward directional light intensity
	InScattering += LFVDirectionalLightColor * HenyeyGreensteinPhase(-FogInstance.PhaseG, dot(RayDirWorld, LFVDirectionalLightDirection));

	// Sky light
	if (View.SkyLightVolumetricScatteringIntensity > 0)
	{
		float3 SkyLighting = View.SkyLightColor.rgb * GetSkySHDiffuseSimple(RayDirWorld * -FogInstance.PhaseG);

		const float SkyVisibility = 1.0f; // Not possible to use AO with analytical integration
		InScattering += (SkyVisibility * View.SkyLightVolumetricScatteringIntensity) * SkyLighting;
	}
	
	// Apply the medium uniform single scattering albedo
	InScattering *= FogInstance.Albedo;

	// Now account for emissive luminance after albedo and heighfog contribution has been accounted for.
	// This do mean that the emissive color follows the medium density variation.
	InScattering += FogInstance.Emissive;

	// Then scale the contribution according to the normalised pre intergrated scattering.
	return InScattering * FogData.IntegratedLuminanceFactor;
}



///////////////////////////////////////////////////////////////////////////////
// Local Fog Volume Instance Evaluation

// Returns color transmittance
float4 GetLFVInstanceContribution(
	in uint InstanceIndex,
	in float3 CamRayTranslatedWorldOrigin,
	in float3 CamRayTranslatedWorldDir,
	in float3 DepthBufferTranslatedWorldPos,
	in bool bPlatformSupportsVolumetricFogOntranslucent)
{
	FLocalFogVolumeGPUInstanceData FogInstance = GetLocalFogVolumeGPUInstanceData(InstanceIndex);

	float3 LuminanceColor = float3(0.0f, 0.0f, 0.0f);
	float Transmittance = 1.0f;

	// The "U" prefix is for all computation done in the Unit Sphere space.
	float3 DepthBufferPosU	= mul(DepthBufferTranslatedWorldPos - FogInstance.TranslatedWorlPos,	FogInstance.PreTranslatedInvTransform);
	float3 RayPosU			= mul(CamRayTranslatedWorldOrigin - FogInstance.TranslatedWorlPos,		FogInstance.PreTranslatedInvTransform);
	float3 RayDirU			= mul(CamRayTranslatedWorldDir,											FogInstance.PreTranslatedInvTransform);
	RayDirU.xyz				= normalize(RayDirU);

	float2 TsU = RayIntersectSphere(RayPosU, RayDirU, float4(0.0, 0.0, 0.0, 1.0));

	//Clamp to make sure we only consider intersection in front of the camera
	const float LFVStartDistance = max(LFVGlobalStartDistance, FogInstance.StartDistance);
	TsU = max(LFVStartDistance * FogInstance.UniformScaleInv, TsU);

	// Clamp the traced distance to the depth buffer and compute world length.
	float3 ViewToDepthVector = DepthBufferPosU - RayPosU;
	float ViewToDepthVectorSqrLength = dot(ViewToDepthVector, ViewToDepthVector);
	float LengthDepthBufferU = sqrt(ViewToDepthVectorSqrLength);
	TsU = min(TsU, LengthDepthBufferU);

	if (any(TsU > 0.0))
	{
		// Make the ray start after the volumetric fog far distance
		if (LFVRenderInVolumetricFog && bPlatformSupportsVolumetricFogOntranslucent)
		{
			// Volumetric fog covers up to MaxDistance along ViewZ, exclude analytical fog from this range
			float CosAngle = dot(CamRayTranslatedWorldDir, View.ViewForward);
			float InvCosAngle = (CosAngle > 0.0001) ? rcp(CosAngle) : 0;
			float ExcludeDistance = View.VolumetricFogMaxDistance * InvCosAngle * FogInstance.UniformScaleInv;
			TsU = max(TsU, ExcludeDistance);
		}

		float RayTracedLengthU = max(0.0, abs(TsU.y - TsU.x));

		if (RayTracedLengthU > 0.0)
		{
			FFogData FogData = LocalFogVolumeEvaluateAnalyticalIntegral(FogInstance, RayPosU + RayDirU * TsU.x, RayDirU, RayTracedLengthU);

			LuminanceColor = LocalFogVolumeEvaluateInScattering(FogInstance, FogData, CamRayTranslatedWorldDir);
			Transmittance = 1.0 - FogData.Coverage;
		}
	}

	return float4(LuminanceColor, Transmittance);
}



// Evaluate the local fog volume contribution for a world position.
// Returns a float4 with Luminance Color in RGB and Transmittance in A.
float4 GetLFVContribution(
	in ViewState ResolvedView,
	in uint2 TilePos,
	in float3 DepthBufferTranslatedWorldPos,
	in bool bPlatformSupportsVolumetricFogOntranslucent,
	inout uint OutLFVCount)
{
	const float3 CamRayTranslatedWorldOrigin= ResolvedView.TranslatedWorldCameraOrigin;
	const float3 CamRayWorldDir				= normalize(DepthBufferTranslatedWorldPos - CamRayTranslatedWorldOrigin);

	OutLFVCount = LFVTileDataTex[uint3(TilePos, 0)];
	
	float3 LFVLuminanceColor = 0.0;
	float LFVTransmittance = 1.0;
	for (int LFVIndex = 0; LFVIndex < OutLFVCount; ++LFVIndex)
	{
		uint InstanceIndex = LFVTileDataTex[uint3(TilePos, 1 + LFVIndex)];

		float4 InstanceContribution = GetLFVInstanceContribution(
			InstanceIndex,
			CamRayTranslatedWorldOrigin,
			CamRayWorldDir,
			DepthBufferTranslatedWorldPos,
			bPlatformSupportsVolumetricFogOntranslucent);

		LFVLuminanceColor = LFVLuminanceColor * InstanceContribution.a + InstanceContribution.rgb;
		LFVTransmittance = LFVTransmittance * InstanceContribution.a;
	}

	return float4(LFVLuminanceColor, LFVTransmittance);
}

float4 GetLFVContribution(
	in ViewState ResolvedView,
	in uint2 TilePos,
	in float3 TranslatedWorldPosition,
	in bool bPlatformSupportsVolumetricFogOntranslucent = true)
{
	uint OutLFVCount = 0;
	return GetLFVContribution(ResolvedView, TilePos, TranslatedWorldPosition, bPlatformSupportsVolumetricFogOntranslucent, OutLFVCount);
}


