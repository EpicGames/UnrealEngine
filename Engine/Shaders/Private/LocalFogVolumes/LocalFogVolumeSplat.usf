// Copyright Epic Games, Inc. All Rights Reserved.
#include "../Common.ush"

#if SHADING_PATH_MOBILE
#define MobileSceneTextures MobileBasePass.SceneTextures 
#endif

#include "../SceneTexturesCommon.ush"

#include "LocalFogVolumeCommon.ush"

#ifndef LFV_TILED_VS
#define LFV_TILED_VS 0
#endif
#ifndef LFV_TILED_PS
#define LFV_TILED_PS 0
#endif
#ifndef LFV_HALFRES_PS
#define LFV_HALFRES_PS 0
#endif
#ifndef LFV_VISUALIZATION_PS
#define LFV_VISUALIZATION_PS 0
#endif



struct FLocalFogVolumeVertexOutput
{
//#if LFV_TILED_VS || LFV_TILED_PS
	nointerpolation uint TileData : TEXCOORD0;
//#endif
};



#if LFV_TILED_VS

Buffer<uint> TileDataBuffer;
float StartDepthZ;

void LocalFogVolumeTiledVS(
	in uint VertexIndex : SV_VertexID,
	in uint InstanceIndex : SV_InstanceID,
	out float4 OutPosition : SV_POSITION,
	out FLocalFogVolumeVertexOutput OtherVertexOutput
)
{
	ResolvedView = ResolveView();

	uint PackedTileCoord = TileDataBuffer[InstanceIndex];
	uint2 TileCoord = LFVUnpackTile(PackedTileCoord);

	uint2 TileVertex = TileCoord * LFVTilePixelSize;
	TileVertex.x += VertexIndex == 1 || VertexIndex == 2 || VertexIndex == 4 ? LFVTilePixelSize : 0;
	TileVertex.y += VertexIndex == 2 || VertexIndex == 4 || VertexIndex == 5 ? LFVTilePixelSize : 0;

	OutPosition = float4(float2(TileVertex) * ResolvedView.ViewSizeAndInvSize.zw * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), StartDepthZ, 1.0f);

	OtherVertexOutput.TileData = PackedTileCoord;
}
#endif // LFV_TILED_VS


#if LFV_TILED_PS
int LocalFogVolumeTileDebug;

#if LFV_VISUALIZATION_PS
#include "../ColorMap.ush"
#endif

void LocalFogVolumeTiledPS(
	in float4 SVPos : SV_POSITION,
#if LFV_HALFRES_PS
	out float4 OutColorThroughput : SV_Target0,
	out float OutDepth : SV_Target1)
#else
	in FLocalFogVolumeVertexOutput OtherVertexOutput,
	out float4 OutColor : SV_Target0)
#endif
{
	ResolvedView = ResolveView();

#if LFV_HALFRES_PS
	const float2 HalfResScale = ResolvedView.ViewSizeAndInvSize.xy * LFVHalfResTextureSizeAndInvSize.zw;
#else
	const float2 HalfResScale = 1.0f;
#endif
	const float2 ViewRectSVPos = SVPos.xy * HalfResScale;  

#if SHADING_PATH_MOBILE
	float2 UVDepth = ViewRectSVPos * ResolvedView.BufferSizeAndInvSize.zw;
	float DeviceZ = LookupDeviceZ(UVDepth); // LookupDeviceZ with uint2(SVPos.xy) as input does not work on mobile.
#else
	float DeviceZ = LookupDeviceZ(uint2(SVPos.xy));
#endif

	DeviceZ = max(0.000000000001, DeviceZ);	// Make sure SvPositionToWorld does not return bad values when DeviceZ is far=0 when using inverted z
	float3 DepthBufferTranslatedWorldPos = SvPositionToResolvedTranslatedWorld(float4(ViewRectSVPos, DeviceZ, 1.0));

#if LFV_HALFRES_PS
	uint2 TilePos = ViewRectSVPos / LFVTilePixelSize;
#else
	uint2 TilePos = LFVUnpackTile(OtherVertexOutput.TileData);
#endif

	uint LFVCount = 0;
	const bool bPlatformSupportsVolumetricFogOntranslucent = false;
	float4 LFVContribution = GetLFVContribution(ResolvedView, TilePos, DepthBufferTranslatedWorldPos, bPlatformSupportsVolumetricFogOntranslucent, LFVCount);

	bool bDoClip = true;
#if LFV_VISUALIZATION_PS
	bDoClip = LocalFogVolumeTileDebug != 1;
#endif
#if LFV_HALFRES_PS
	bDoClip = false; // We want to write all pixel to avoid clear operation
#endif

	const float4 IdentityLuminanceTransmittance = float4(0.0, 0.0, 0.0, 1.0);
	if (all(LFVContribution == IdentityLuminanceTransmittance) && bDoClip)
	{
		clip(-1.0f);
	}

#if LFV_VISUALIZATION_PS == 1

	if (LocalFogVolumeTileDebug > 0)
	{
		float Coverage = 0.5;
		float3 DebugColor = GetHSVDebugColor(float(LFVCount) / 4.0f);
		DebugColor = DebugColor * Coverage; // * PrimaryView.OneOverPreExposure * PrimaryView.PreExposure == 1

		OutColor = float4(DebugColor, 1.0 - Coverage);
	}

#else // LFV_VISUALIZATION_PS

#if LFV_HALFRES_PS
	OutColorThroughput = float4(
		LFVContribution.rgb * PrimaryView.PreExposure,
		LFVContribution.a);
	OutDepth = ConvertFromDeviceZ(DeviceZ) * CENTIMETER_TO_METER; // Allows to represent depth up to 65 km while keeping enough accuracy close to the camera
#else
	OutColor = float4(
		LFVContribution.rgb * PrimaryView.PreExposure,
		LFVContribution.a);
#endif

#endif // LFV_VISUALIZATION_PS

}
#endif // LFV_TILED_PS

