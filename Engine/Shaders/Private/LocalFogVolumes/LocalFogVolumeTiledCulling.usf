// Copyright Epic Games, Inc. All Rights Reserved.
#include "../Common.ush"

#include "LocalFogVolumeCommon.ush"
#include "../Nanite/NaniteHZBCull.ush"
#include "../HZB.ush"

float4 LeftPlane;
float4 RightPlane;
float4 TopPlane;
float4 BottomPlane;
float4 NearPlane;
float2 ViewToTileSpaceRatio;

Buffer<float4> LocalFogVolumeCullingDataBuffer;

RWTexture2DArray<uint> LocalFogVolumeTileDataTextureUAV;

RWBuffer<uint> TileDataBufferUAV;
RWBuffer<uint> TileDrawIndirectBufferUAV;

#ifndef THREADGROUP_SIZE
#error THREADGROUP_SIZE must be defined
#endif

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void LocalFogVolumeTiledCullingCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	uint2 TileCoord = uint2(DispatchThreadId.xy);

	if (any(TileCoord >= LFVTileDataResolution))
	{
		return; // Skip out of bound texture tile data 
	}

	// Init primitive index
	if (DispatchThreadId.x == 0 && DispatchThreadId.y == 0)
	{
		const uint IndexCountPerInstance = 6; // LFV_TODO handle rect primitives with 4
		TileDrawIndirectBufferUAV[0] = IndexCountPerInstance;
	}

	float2 MinTileLerp =  float2(TileCoord)       / float2(LFVTileDataResolution);
	float2 MaxTileLerp = (float2(TileCoord)+1.0f) / float2(LFVTileDataResolution);

	// Apply the view to tile space ratio to account ofr the fact that tiles can span outside the view pixel due to their discretised nature.
	MinTileLerp *= ViewToTileSpaceRatio;
	MaxTileLerp *= ViewToTileSpaceRatio;
	// Still clamp MaxTileLerp to not include not visible volumes to the right and up part of the screen.
	MaxTileLerp = min(MaxTileLerp, 1.0f.xx);

	float4 TilePlanes[5];
	TilePlanes[0] = lerp(LeftPlane, RightPlane, MinTileLerp.xxxx);		// Left tile plane
	TilePlanes[1] = lerp(LeftPlane, RightPlane, MaxTileLerp.xxxx);		// Right tile plane
	TilePlanes[2] = lerp(-TopPlane, -BottomPlane, MinTileLerp.yyyy);	// Bottom tile plane
	TilePlanes[3] = lerp(-TopPlane, -BottomPlane, MaxTileLerp.yyyy);	// Top plane
	TilePlanes[4] = NearPlane;											// Near plane
	// Swap right and top plane to go from lerpable planes to planes we can use for culling
	TilePlanes[1] *= -1.0f;
	TilePlanes[3] *= -1.0f;

#if 0
	// Optional plane normalise
	UNROLL
	for (uint i = 0; i < 5; ++i)
	{
		float SquareSum = TilePlanes[i].x * TilePlanes[i].x + TilePlanes[i].y * TilePlanes[i].y + TilePlanes[i].z * TilePlanes[i].z;
		float Scale = 1.0 / sqrt(SquareSum);
		TilePlanes[i] *= Scale;
	}
#endif
#if USE_HZB
	static float4x4 IdentityLocalToTranslatedWorldMatrix =
	{
		{ 1, 0, 0, 0 },
		{ 0, 1, 0, 0 },
		{ 0, 0, 1, 0 },
		{ 0, 0, 0, 1 }
	};
#endif

	uint LFVCount = 0;
	for (int LFVIndex = 0; LFVIndex < LFVInstcount; ++LFVIndex)
	{
		float4 Sphere = LocalFogVolumeCullingDataBuffer[LFVIndex];

		bool bCompletelyOutside = false;
		UNROLL
		for (uint i = 0; i < 5; ++i)
		{
			if (dot(TilePlanes[i].xyz, Sphere.xyz) - TilePlanes[i].w > Sphere.w)
			{
				bCompletelyOutside = true;
				break;
			}
		}

	#if USE_HZB
		if (!bCompletelyOutside)
		{
		#if 0
			// Sphere
			float4 SphereCenterView = mul(Sphere.xyz + LWCToFloat(PrimaryView.PreViewTranslation), ResolvedView.TranslatedWorldToView);
			FFrustumCullData Cull = SphereCullFrustum(SphereCenterView, Sphere.w, ResolvedView.ViewToClip, 0.0/*float ZNear*/, false/*bool bNearClip*/ , true/*bool bSkipFrustumCull*/);
		#else
			FFrustumCullData Cull = BoxCullFrustum(
				Sphere.xyz + LWCToFloat(PrimaryView.PreViewTranslation),
				Sphere.www,
				IdentityLocalToTranslatedWorldMatrix,
				ResolvedView.TranslatedWorldToClip,
				ResolvedView.ViewToClip,
				false /*bIsOrtho*/, true /* near clip */, true /* skip culling */
			);
		#endif

		#if 0 // USE_TILED_HZB // Not working yet...
			Cull.RectMin.xy = float2(TileCoord.x,			(TileCoord.y))			/ float2(LFVTileDataResolution);
			Cull.RectMax.xy = float2(TileCoord.x + 1.0f,	(TileCoord.y + 1.0f))	/ float2(LFVTileDataResolution);

			Cull.RectMin.x = Cull.RectMin.x * ViewToTileSpaceRatio.x * 2.0 - 1.0;
			Cull.RectMax.x = Cull.RectMax.x * ViewToTileSpaceRatio.x * 2.0 - 1.0;

			Cull.RectMin.y = 1.0/ViewToTileSpaceRatio.y - Cull.RectMin.y;
			Cull.RectMax.y = 1.0/ViewToTileSpaceRatio.y - Cull.RectMax.y;

			Cull.RectMin.y = Cull.RectMin.y * 2 - 1;
			Cull.RectMax.y = Cull.RectMax.y * 2 - 1;
		#endif

			int4 ActualHZBViewRect = int4(0.0.xx, ResolvedView.ViewRectMinAndSize.zw);
			FScreenRect Rect = GetScreenRect(ActualHZBViewRect, Cull, 4);

			bCompletelyOutside = !IsVisibleHZB(Rect, true /*bSample4x4*/);
		}
	#endif

		if (!bCompletelyOutside)
		{
			LocalFogVolumeTileDataTextureUAV[uint3(TileCoord, 1 + LFVCount)] = LFVIndex;
			LFVCount++;
		}
	}

	LocalFogVolumeTileDataTextureUAV[uint3(TileCoord, 0)] = LFVCount;

	if (LFVCount > 0)
	{
		uint WriteToIndex;
		InterlockedAdd(TileDrawIndirectBufferUAV[1], 1, WriteToIndex);

		TileDataBufferUAV[WriteToIndex] = LFVPackTile(TileCoord);
	}
}


