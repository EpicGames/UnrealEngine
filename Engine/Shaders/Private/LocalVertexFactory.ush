// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalVertexFactory.ush: Local vertex factory shader code.
=============================================================================*/

#include "VertexFactoryCommon.ush"
#include "LocalVertexFactoryCommon.ush"
#include "LightmapData.ush"
#include "GpuSkinCommon.ush"
#include "SplineMeshCommon.ush"

#include "/Engine/Generated/UniformBuffers/PrecomputedLightingBuffer.ush"

#if USE_INSTANCING
float4 InstancingOffset;
uint InstanceOffset;
#endif

#ifndef SUPPORT_GPUSKIN_PASSTHROUGH
#define SUPPORT_GPUSKIN_PASSTHROUGH 0
#endif

#ifndef MANUAL_VERTEX_FETCH
#define MANUAL_VERTEX_FETCH 0
#endif

#define VF_ColorIndexMask_Index 0
#define VF_NumTexcoords_Index 1
#define FV_LightMapIndex_Index 2
#define VF_VertexOffset 3

#if MANUAL_VERTEX_FETCH
	Buffer<float4> VertexFetch_InstanceOriginBuffer;
	Buffer<float4> VertexFetch_InstanceTransformBuffer;
	Buffer<float4> VertexFetch_InstanceLightmapBuffer;

	#define VF_REQUIRES_HITPROXY_INDIRECTION 1
	int VertexFactoryGetVertexFetchParameter(int ParameterIndex)
	{
		return LocalVF.VertexFetch_Parameters[ParameterIndex];
	}
#endif //! MANUAL_VERTEX_FETCH

#define VF_REQUIRES_HITPROXY_INDIRECTION 1

/**
 * Per-vertex inputs from bound vertex buffers
 */
struct FVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

#if !MANUAL_VERTEX_FETCH
	#if METAL_ES3_1_PROFILE
		float3	TangentX	: ATTRIBUTE1;
		// TangentZ.w contains sign of tangent basis determinant
		float4	TangentZ	: ATTRIBUTE2;

		float4	Color		: ATTRIBUTE3;
	#else
		HALF3_TYPE	TangentX	: ATTRIBUTE1;
		// TangentZ.w contains sign of tangent basis determinant
		HALF4_TYPE	TangentZ	: ATTRIBUTE2;

		HALF4_TYPE	Color		: ATTRIBUTE3;
	#endif
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	#if !MANUAL_VERTEX_FETCH
			// These used to be packed texcoord arrays, but these cause problems with alighnment on some Vulkan drivers
			#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
				float4	TexCoords0 : ATTRIBUTE4;
			#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 1
				float2	TexCoords0 : ATTRIBUTE4;
			#endif

			#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
				float4	TexCoords1 : ATTRIBUTE5;
			#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 3
				float2	TexCoords1 : ATTRIBUTE5;
			#endif

			#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5
				float4	TexCoords2 : ATTRIBUTE6;
			#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 5
				float2	TexCoords2 : ATTRIBUTE6;
			#endif

			#if NUM_MATERIAL_TEXCOORDS_VERTEX > 7
				float4	TexCoords3 : ATTRIBUTE7;
			#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 7
				float2	TexCoords3 : ATTRIBUTE7;
			#endif
		#endif
#elif USE_PARTICLE_SUBUVS && !MANUAL_VERTEX_FETCH
	float2	TexCoords0 : ATTRIBUTE4;
#endif
	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE13
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(13)

#if USE_INSTANCING && !MANUAL_VERTEX_FETCH
	float4 InstanceOrigin : ATTRIBUTE8;  // per-instance random in w 
	half4 InstanceTransform1 : ATTRIBUTE9;  // hitproxy.r + 256 * selected in .w
	half4 InstanceTransform2 : ATTRIBUTE10; // hitproxy.g in .w
	half4 InstanceTransform3 : ATTRIBUTE11; // hitproxy.b in .w
	float4 InstanceLightMapAndShadowMapUVBias : ATTRIBUTE12; 
#endif //USE_INSTANCING

#if USE_INSTANCING
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId	: SV_InstanceID;
#else
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()

#if SUPPORT_GPUSKIN_PASSTHROUGH && !MANUAL_VERTEX_FETCH
	float4	PreSkinPosition	: ATTRIBUTE14;
#endif

#if NEEDS_LIGHTMAP_COORDINATE && !MANUAL_VERTEX_FETCH
	float2	LightMapCoordinate : ATTRIBUTE15;
#endif

	uint VertexId : SV_VertexID;
};

bool IsGPUSkinPassThrough()
{
	#if SUPPORT_GPUSKIN_PASSTHROUGH
		return LocalVF.bIsGPUSkinPassThrough;
	#else
		return false;
	#endif
}

#if RAYHITGROUPSHADER || COMPUTESHADER
uint GetNumRayTracingDynamicMeshVerticesIndirect()
{
	return 0;
}
#endif

#if RAYHITGROUPSHADER

FVertexFactoryInput LoadVertexFactoryInputForHGS(uint TriangleIndex, int VertexIndex)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(TriangleIndex);

	Input.VertexId = Tri.Indices[VertexIndex];
	Input.Position = float4(Tri.LocalPositions[VertexIndex], 1.0f);
#if VF_USE_PRIMITIVE_SCENE_DATA
	const uint GPUSceneInstanceId = GetInstanceUserData();
	const FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, GPUSceneInstanceId, InstanceSceneData.RelativeId);

#if USE_INSTANCING
	Input.InstanceId = InstanceSceneData.RelativeId;
#endif
#else
	#error "HGS requires GPU Scene support"
#endif // VF_USE_PRIMITIVE_SCENE_DATA

	return Input;
}
#endif


#if COMPUTESHADER && MANUAL_VERTEX_FETCH

FVertexFactoryInput LoadVertexFactoryInputForDynamicUpdate(uint TriangleIndex, int VertexIndex, uint PrimitiveId, uint DrawInstanceId)
{
	FVertexFactoryInput Input = (FVertexFactoryInput)0;

	if (IsGPUSkinPassThrough())
	{
		// GPUSkinPassThrough can change the PositionBuffer every frame but doesn't always update the corresponding 
		// LocalVF.VertexFetch_PositionBuffer since it relies on fixed function vertex buffer fetch and so doesn't need
		// to pay the cost of recreating the UniformBuffer.
		// For the compute shader case we store the PositionBuffer SRV in the loose uniform buffer.
		Input.VertexId = TriangleIndex * 3 + VertexIndex;
		uint VertexOffset = LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId;
		Input.Position.x = GPUSkinPassThroughVFLooseParameters.PositionBuffer[VertexOffset * 3 + 0];
		Input.Position.y = GPUSkinPassThroughVFLooseParameters.PositionBuffer[VertexOffset * 3 + 1];
		Input.Position.z = GPUSkinPassThroughVFLooseParameters.PositionBuffer[VertexOffset * 3 + 2];
	}
	else
	{
		Input.VertexId = TriangleIndex * 3 + VertexIndex;
		uint VertexOffset = LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId * 3;
		Input.Position.x = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 0];
		Input.Position.y = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 1];
		Input.Position.z = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 2];
	}

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);
	VF_GPUSCENE_SET_INPUT_FOR_RT(Input, PrimitiveData.InstanceSceneDataOffset + DrawInstanceId, DrawInstanceId);

#if USE_SPLINEDEFORM
	Input.Position.w = 1.0;
#endif

	return Input;
}

#endif

/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-only shader.
 */
struct FPositionOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE1
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)

#if USE_INSTANCING && !MANUAL_VERTEX_FETCH
	float4 InstanceOrigin : ATTRIBUTE8;  // per-instance random in w 
	half4 InstanceTransform1 : ATTRIBUTE9;  // hitproxy.r + 256 * selected in .w
	half4 InstanceTransform2 : ATTRIBUTE10; // hitproxy.g in .w
	half4 InstanceTransform3 : ATTRIBUTE11; // hitproxy.b in .w
#endif	// USE_INSTANCING

#if USE_INSTANCING
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId : SV_InstanceID;
#else
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 

#if MANUAL_VERTEX_FETCH
	uint VertexId : SV_VertexID;
#endif
};

/** 
 * Per-vertex inputs from bound vertex buffers.  Used by passes with a trimmed down position-and-normal-only shader.
 */
struct FPositionAndNormalOnlyVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	float4	Normal		: ATTRIBUTE2;

	// Dynamic instancing related attributes with InstanceIdOffset : ATTRIBUTE1
	VF_GPUSCENE_DECLARE_INPUT_BLOCK(1)

#if USE_INSTANCING && !MANUAL_VERTEX_FETCH
	float4 InstanceOrigin : ATTRIBUTE8;  // per-instance random in w 
	half4 InstanceTransform1 : ATTRIBUTE9;  // hitproxy.r + 256 * selected in .w
	half4 InstanceTransform2 : ATTRIBUTE10; // hitproxy.g in .w
	half4 InstanceTransform3 : ATTRIBUTE11; // hitproxy.b in .w
#endif	// USE_INSTANCING

#if USE_INSTANCING
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId : SV_InstanceID;
#else
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 

#if MANUAL_VERTEX_FETCH
	uint VertexId : SV_VertexID;
#endif
};

/** The per-instance data that is either manually-fetched or included with the input vertex data */
struct FVertexFactoryInstanceInput
{
	float4 Transform1;
	float4 Transform2;
	float4 Transform3;
	float4 Origin;
};

/** Intermediates that are common to all supported vertex factory input types */
struct FVertexFactoryIntermediatesCommon
{
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
#if USE_INSTANCING || USE_INSTANCE_CULLING
	FVertexFactoryInstanceInput InstanceInput;
#endif
#if USE_SPLINEDEFORM
	FSplineMeshShaderParams SplineMeshParams;
#endif
};

/** 
 * Caches intermediates that would otherwise have to be computed multiple times.  Avoids relying on the compiler to optimize out redundant operations.
 */
struct FVertexFactoryIntermediates
{
	FVertexFactoryIntermediatesCommon Common;
#if VF_USE_PRIMITIVE_SCENE_DATA
	int PrimitiveLocalInstanceIndex;
#endif

	half3x3 TangentToLocal;
	half3x3 TangentToWorld;
	half TangentToWorldSign;

	half4 Color;

	bool bEvaluateWorldPositionOffset;

#if USE_INSTANCE_CULLING
	float4 HitProxyId;
#endif

	float IsVisible;
#if USE_INSTANCING
	float4 InstanceLightMapAndShadowMapUVBias;
#endif
#if USE_INSTANCING || USE_INSTANCE_CULLING
	// x = per-instance fade out amount, y = hide/show flag, z dither fade cutoff, w - CustomData index
	float4 PerInstanceParams;
#endif

	float3 PreSkinPosition;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediatesCommon Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return GetPrimitiveData(Intermediates.Common);
}

FInstanceSceneData GetInstanceData(FVertexFactoryIntermediatesCommon Intermediates)
{
	return Intermediates.SceneData.InstanceData;
}

FInstanceSceneData GetInstanceData(FVertexFactoryIntermediates Intermediates)
{
	return GetInstanceData(Intermediates.Common);
}

#if USE_SPLINEDEFORM

#if VF_USE_PRIMITIVE_SCENE_DATA && FEATURE_LEVEL > FEATURE_LEVEL_ES3_1

FSplineMeshShaderParams GetSplineMeshParams(FInstanceSceneData InstanceData)
{
	return SplineMeshLoadParamsFromInstancePayload(InstanceData);
}

#else

// These currently still need to be bound loosely when not using GPUScene
FSplineMeshShaderParams GetSplineMeshParams(FInstanceSceneData InstanceData)
{
	return UnpackSplineMeshParams(SplineMeshVFPrimitiveShaderParameters.SplineParams);
}

#endif // VF_USE_PRIMITIVE_SCENE_DATA && FEATURE_LEVEL > FEATURE_LEVEL_ES3_1

#endif // USE_SPLINEDEFORM

FVertexFactoryInstanceInput MakeInstanceInput(float4 Transform1, float4 Transform2, float4 Transform3, float4 Origin)
{
	FVertexFactoryInstanceInput Result;
	Result.Transform1 = Transform1;
	Result.Transform2 = Transform2;
	Result.Transform3 = Transform3;
	Result.Origin = Origin;

	return Result;
}

FVertexFactoryInstanceInput InitInstanceInput()
{
	return MakeInstanceInput(
		float4(1, 0, 0, 0),
		float4(0, 1, 0, 0),
		float4(0, 0, 1, 0),
		float4(0, 0, 0, 0)
	);
}

#if USE_INSTANCING

#if MANUAL_VERTEX_FETCH

FVertexFactoryInstanceInput ManualFetchInstanceInput(uint InstanceId)
{
	FVertexFactoryInstanceInput Result = (FVertexFactoryInstanceInput)0;
	Result.Transform1 = InstanceVF.VertexFetch_InstanceTransformBuffer[3 * (InstanceId + InstanceOffset) + 0];
	Result.Transform2 = InstanceVF.VertexFetch_InstanceTransformBuffer[3 * (InstanceId + InstanceOffset) + 1];
	Result.Transform3 = InstanceVF.VertexFetch_InstanceTransformBuffer[3 * (InstanceId + InstanceOffset) + 2];
	Result.Origin = InstanceVF.VertexFetch_InstanceOriginBuffer[(InstanceId + InstanceOffset)];

	return Result;
}

#define LOCALVF_GET_INSTANCE_INPUT(VFInput) ManualFetchInstanceInput(GetInstanceId(GetInstanceIdFromVF(VFInput)))

#else

#define LOCALVF_GET_INSTANCE_INPUT(VFInput) MakeInstanceInput( \
	VFInput.InstanceTransform1, \
	VFInput.InstanceTransform2, \
	VFInput.InstanceTransform3, \
	VFInput.InstanceOrigin)

#endif // MANUAL_VERTEX_FETCH

float4x4 GetInstanceTransform(FVertexFactoryInstanceInput InstanceInput)
{
	return float4x4(
		float4(InstanceInput.Transform1.xyz, 0.0f),
		float4(InstanceInput.Transform2.xyz, 0.0f),
		float4(InstanceInput.Transform3.xyz, 0.0f),
		float4(InstanceInput.Origin.xyz, 1.0f)
	);
}

float4x4 GetInstanceTransform(FVertexFactoryIntermediatesCommon Intermediates)
{
	return GetInstanceTransform(Intermediates.InstanceInput);
}

float4x4 GetInstanceTransform(FVertexFactoryIntermediates Intermediates)
{
	return GetInstanceTransform(Intermediates.Common);
}

float4x4 GetInstancePrevTransform(FVertexFactoryIntermediates Intermediates)
{
	// Assumes instance transform never change, which means per-instance motion will cause TAA and motion blur artifacts.
	return GetInstanceTransform(Intermediates);
}

float4x4 GetInstanceTransform(FPositionOnlyVertexFactoryInput Input)
{
	return GetInstanceTransform(LOCALVF_GET_INSTANCE_INPUT(Input));
}

float4x4 GetInstanceTransform(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	return GetInstanceTransform(LOCALVF_GET_INSTANCE_INPUT(Input));
}

half3x3 GetInstanceToLocal3x3(FVertexFactoryIntermediates Intermediates)
{
	return (half3x3)GetInstanceTransform(Intermediates);
}

float2 GetInstanceShadowMapBias(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.InstanceLightMapAndShadowMapUVBias.zw;
}

float2 GetInstanceLightMapBias(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.InstanceLightMapAndShadowMapUVBias.xy;
}

float GetInstanceSelected(FVertexFactoryIntermediates Intermediates)
{
	return trunc(Intermediates.Common.InstanceInput.Transform1.w * (1.0 / 256.0));
}

float GetInstanceRandom(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Common.InstanceInput.Origin.w;
}

float3 GetInstanceOrigin(FVertexFactoryIntermediates Intermediates)
{
	float3 Offset = mul(InstancingOffset.xyz, GetInstanceToLocal3x3(Intermediates));
	return Intermediates.Common.InstanceInput.Origin.xyz + Offset;
}

#else // !USE_INSTANCING

// Stub this for non-instancing
#define LOCALVF_GET_INSTANCE_INPUT(VFInput) InitInstanceInput()

#endif	// USE_INSTANCING

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for( int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++ )
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

#if USE_PARTICLE_SUBUVS
	// Output TexCoord0 for when previewing materials that use ParticleSubUV.
	Result.Particle.SubUVCoords[0] = GetUV(Interpolants, 0);
	Result.Particle.SubUVCoords[1] = GetUV(Interpolants, 0);
#endif	// USE_PARTICLE_SUBUVS

	half3 TangentToWorld0 = GetTangentToWorld0(Interpolants).xyz;
	half4 TangentToWorld2 = GetTangentToWorld2(Interpolants);
	Result.UnMirrored = TangentToWorld2.w;

	Result.VertexColor = GetColor(Interpolants);

#if USE_INSTANCING || USE_INSTANCE_CULLING 
	#if	NEEDS_PER_INSTANCE_PARAMS
		Result.PerInstanceParams = Interpolants.PerInstanceParams;
	#else
		Result.PerInstanceParams = float4(1, 1, 0, 0);
	#endif
#endif

	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

#if LIGHTMAP_UV_ACCESS
#if NEEDS_LIGHTMAP_COORDINATE
	#if (ES3_1_PROFILE)
		// Not supported in pixel shader
		Result.LightmapUVs = float2(0, 0);
	#else
		Result.LightmapUVs = Interpolants.LightMapCoordinate.xy;
	#endif	// ES3_1_PROFILE
#endif	// NEEDS_LIGHTMAP_COORDINATE
#endif	// LIGHTMAP_UV_ACCESS

	Result.TwoSidedSign = 1;

#if VF_USE_PRIMITIVE_SCENE_DATA
	FInstanceSceneData InstanceData = GetInstanceSceneData(Interpolants.InstanceId);
	Result.PrimitiveId = InstanceData.PrimitiveId;

#if USES_PER_INSTANCE_CUSTOM_DATA
	Result.CustomDataOffset = InstanceData.CustomDataOffset;
	Result.CustomDataCount  = InstanceData.CustomDataCount;
#endif
	Result.PerInstanceRandom = InstanceData.RandomID;
#else
	Result.PrimitiveId = 0;
#endif

#if LOCAL_VF_INTERPOLATE_CUSTOM_DATA_OFFSET
	Result.CustomDataOffset = Interpolants.CustomDataOffset;
	Result.CustomDataCount  = Interpolants.CustomDataCount;
#endif

#if LOCAL_VF_INTERPOLATE_PER_INSTANCE_RANDOM
	Result.PerInstanceRandom = Interpolants.PerInstanceRandom;
#endif

#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS
#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
	Result.InstanceLocalToWorld = GetInstanceLocalToWorld(Interpolants);
#endif
#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
	Result.InstanceWorldToLocal = GetInstanceWorldToLocal(Interpolants);
#endif
#else
#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
	Result.InstanceLocalToWorld = InstanceData.LocalToWorld;
#endif
#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
	Result.InstanceWorldToLocal = InstanceData.WorldToLocal;
#endif
#endif

#if NEEDS_PARTICLE_LOCAL_TO_WORLD || NEEDS_PARTICLE_WORLD_TO_LOCAL
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Result.PrimitiveId);
#endif

#if NEEDS_PARTICLE_LOCAL_TO_WORLD
	Result.Particle.ParticleToWorld = PrimitiveData.LocalToWorld;
#endif

#if NEEDS_PARTICLE_WORLD_TO_LOCAL
	Result.Particle.WorldToParticle = PrimitiveData.WorldToLocal;
#endif

	return Result;
}

half3x3 CalcTangentToWorldNoScale(FVertexFactoryIntermediates Intermediates, half3x3 TangentToLocal)
{
	half3x3 LocalToWorld = DFToFloat3x3(GetInstanceData(Intermediates).LocalToWorld);
	half3 InvScale = GetInstanceData(Intermediates).InvNonUniformScale;
	LocalToWorld[0] *= InvScale.x;
	LocalToWorld[1] *= InvScale.y;
	LocalToWorld[2] *= InvScale.z;
	return mul(TangentToLocal, LocalToWorld); 
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	half3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.Common.SceneData;
	// FIXME: just for compatibility with assets that use custom HLSL expressions, will be removed once we fix up all these assets
	Result.PrimitiveId = Intermediates.Common.SceneData.PrimitiveId;

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Intermediates);
	FInstanceSceneData InstanceData = GetInstanceData(Intermediates);

	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}

#if USE_INSTANCING
	float4x4 InstanceToPrimitive = GetInstanceTransform(Intermediates);
	if (bIsPreviousFrame)
	{
		Result.PositionPrimitiveSpace = mul(float4(WorldPosition, 1), DFFastToTranslatedWorld(PrimitiveData.PreviousWorldToLocal, ResolvedView.PrevPreViewTranslation)).xyz;
	}
	else
	{
		Result.PositionPrimitiveSpace = mul(float4(Result.PositionInstanceSpace, 1), InstanceToPrimitive).xyz;
	}

	Result.InstanceLocalToWorld = DFMultiply(InstanceToPrimitive, PrimitiveData.LocalToWorld);
	#if	NEEDS_PER_INSTANCE_PARAMS
	Result.PerInstanceParams = Intermediates.PerInstanceParams;
	#endif
	Result.InstanceId = GetInstanceId(GetInstanceIdFromVF(Input));
	Result.InstanceOffset = InstanceOffset;
	Result.PrevFrameLocalToWorld = DFMultiply(GetInstancePrevTransform(Intermediates), PrimitiveData.PreviousLocalToWorld);
	Result.PerInstanceRandom = GetInstanceRandom(Intermediates);

	// Calculate derived world to local
	{
		float3 Scale2;
		Scale2.x = length2(InstanceToPrimitive[0].xyz);
		Scale2.y = length2(InstanceToPrimitive[1].xyz);
		Scale2.z = length2(InstanceToPrimitive[2].xyz);

		float3 InvNonUniformScale = rsqrt(Scale2);

		float4x4 LocalToInstance = InstanceToPrimitive;
		LocalToInstance[0].xyz	*= Pow2(InvNonUniformScale.x);
		LocalToInstance[1].xyz	*= Pow2(InvNonUniformScale.y);
		LocalToInstance[2].xyz	*= Pow2(InvNonUniformScale.z);
		LocalToInstance[3].xyz	 = 0.0f;
		LocalToInstance			 = transpose(LocalToInstance);
		LocalToInstance[3].xyz	 = mul(float4(-InstanceToPrimitive[3].xyz, 0.0f), LocalToInstance).xyz;

		Result.InstanceWorldToLocal = DFMultiply(PrimitiveData.WorldToLocal, LocalToInstance);
	}
#else
	Result.PrevFrameLocalToWorld = InstanceData.PrevLocalToWorld;
	#if USE_INSTANCE_CULLING
		if (bIsPreviousFrame)
		{
			Result.PositionPrimitiveSpace = mul(float4(WorldPosition, 1), DFFastToTranslatedWorld(PrimitiveData.PreviousWorldToLocal, ResolvedView.PrevPreViewTranslation)).xyz;
		}
		else
		{
			Result.PositionPrimitiveSpace = mul(float4(WorldPosition, 1), DFFastToTranslatedWorld(PrimitiveData.WorldToLocal, ResolvedView.PreViewTranslation)).xyz;
		}
		Result.InstanceLocalToWorld = InstanceData.LocalToWorld;
		Result.InstanceWorldToLocal = InstanceData.WorldToLocal;
		#if	NEEDS_PER_INSTANCE_PARAMS
		Result.PerInstanceParams = Intermediates.PerInstanceParams;
		#endif
	#else
		Result.PositionPrimitiveSpace = Result.PositionInstanceSpace;
	#endif
	#if USES_PER_INSTANCE_CUSTOM_DATA && VF_USE_PRIMITIVE_SCENE_DATA
		Result.CustomDataOffset = InstanceData.CustomDataOffset;
		Result.CustomDataCount = InstanceData.CustomDataCount;
	#endif
	Result.PerInstanceRandom = InstanceData.RandomID;
#endif

	Result.VertexColor = Intermediates.Color;

	// does not handle instancing!
	Result.TangentToWorld = Intermediates.TangentToWorld;

	Result.bEvaluateWorldPositionOffset = Intermediates.bEvaluateWorldPositionOffset;

	Result.PreSkinnedPosition = Intermediates.PreSkinPosition.xyz;
#if SUPPORT_GPUSKIN_PASSTHROUGH
	if (IsGPUSkinPassThrough())
	{
		Result.PreSkinnedNormal = GPUSkinPassThroughVFLooseParameters.PreSkinnedTangentBuffer[Input.VertexId * 2 + 1].xyz;
	}
	else
#endif
	{
		Result.PreSkinnedNormal = TangentToLocal[2]; //TangentBias(Input.TangentZ.xyz);
	}

	const uint NumFetchTexCoords = LocalVF.VertexFetch_Parameters[VF_NumTexcoords_Index];
#if MANUAL_VERTEX_FETCH && NUM_MATERIAL_TEXCOORDS_VERTEX
		UNROLL
		for (uint CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
		{
			// Clamp coordinates to mesh's maximum as materials can request more than are available
			uint ClampedCoordinateIndex = min(CoordinateIndex, NumFetchTexCoords-1);
			Result.TexCoords[CoordinateIndex] = LocalVF.VertexFetch_TexCoordBuffer[NumFetchTexCoords * (LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + ClampedCoordinateIndex];
		}
#elif NUM_MATERIAL_TEXCOORDS_VERTEX
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
			Result.TexCoords[0] = Input.TexCoords0.xy;
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
			Result.TexCoords[1] = NumFetchTexCoords > 1 ? Input.TexCoords0.zw : Result.TexCoords[0];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
			Result.TexCoords[2] = NumFetchTexCoords > 2 ? Input.TexCoords1.xy : Result.TexCoords[1];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
			Result.TexCoords[3] = NumFetchTexCoords > 3 ? Input.TexCoords1.zw : Result.TexCoords[2];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 4
			Result.TexCoords[4] = NumFetchTexCoords > 4 ? Input.TexCoords2.xy : Result.TexCoords[3];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 5
			Result.TexCoords[5] = NumFetchTexCoords > 5 ? Input.TexCoords2.zw : Result.TexCoords[4];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 6
			Result.TexCoords[6] = NumFetchTexCoords > 6 ? Input.TexCoords3.xy : Result.TexCoords[5];
		#endif
		#if NUM_MATERIAL_TEXCOORDS_VERTEX > 7
			Result.TexCoords[7] = NumFetchTexCoords > 7 ? Input.TexCoords3.zw : Result.TexCoords[6];
		#endif
#endif  //MANUAL_VERTEX_FETCH && NUM_MATERIAL_TEXCOORDS_VERTEX

#if NEEDS_PARTICLE_LOCAL_TO_WORLD
	Result.Particle.ParticleToWorld = PrimitiveData.LocalToWorld;
#endif

#if NEEDS_PARTICLE_WORLD_TO_LOCAL
	Result.Particle.WorldToParticle = PrimitiveData.WorldToLocal;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float4 CalcPositionInstanceSpace(FVertexFactoryIntermediatesCommon Intermediates, float4 Position)
{
#if USE_SPLINEDEFORM && !RAYHITGROUPSHADER
	// NOTE: The #if above does not include raytracing shaders because the cached position already
	// has the spline deformation baked in.
	return float4(SplineMeshDeformLocalPos(Intermediates.SplineMeshParams, Position.xyz), 1);
#else
	return Position;
#endif
}

float4 CalcPositionPrimitiveSpace(FVertexFactoryIntermediatesCommon Intermediates, float4 Position)
{
#if USE_INSTANCING
	return mul(CalcPositionInstanceSpace(Intermediates, Position), GetInstanceTransform(Intermediates));
#else
	return CalcPositionInstanceSpace(Intermediates, Position);
#endif
}

half3x3 CalcTangentToLocal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, inout float TangentSign)
{
#if MANUAL_VERTEX_FETCH
	half3 TangentInputX = LocalVF.VertexFetch_PackedTangentsBuffer[2 * (LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + 0].xyz;
	half4 TangentInputZ = LocalVF.VertexFetch_PackedTangentsBuffer[2 * (LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + 1].xyzw;
#else
	half3 TangentInputX = Input.TangentX;
	half4 TangentInputZ = Input.TangentZ;
#endif

	half3 TangentX = TangentInputX;
	half4 TangentZ = TangentInputZ;
	if (!IsGPUSkinPassThrough())
	{
		TangentX = TangentBias(TangentInputX);
		TangentZ = TangentBias(TangentInputZ);
	}

	TangentSign = TangentZ.w;

#if USE_SPLINEDEFORM
	// Make slice rotation matrix, and use that to transform tangents
#if RAYHITGROUPSHADER
	// When raytracing, Input.Position already has the spline deformation baked in, so we need to
	// go back to the original vertex buffer to compute the SliceRot matrix.
	uint VertexOffset = LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId * 3;
	float3 SliceInputPos = 0;
	SliceInputPos.x = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 0];
	SliceInputPos.y = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 1];
	SliceInputPos.z = LocalVF.VertexFetch_PositionBuffer[VertexOffset + 2];
#else
	float3 SliceInputPos = Input.Position.xyz;
#endif
	half3x3 SliceRot = SplineMeshCalcSliceRotFromLocalPos(Intermediates.Common.SplineMeshParams, SliceInputPos.xyz);

	TangentX = mul(TangentX, SliceRot);
	TangentZ.xyz = mul(TangentZ.xyz, SliceRot);
#endif	// USE_SPLINEDEFORM

	// derive the binormal by getting the cross product of the normal and tangent
	half3 TangentY = cross(TangentZ.xyz, TangentX) * TangentZ.w;
	
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	half3x3 Result;
	Result[0] = cross(TangentY, TangentZ.xyz) * TangentZ.w;
	Result[1] = TangentY;
	Result[2] = TangentZ.xyz;

	return Result;
}

half3x3 CalcTangentToWorld(FVertexFactoryIntermediates Intermediates, half3x3 TangentToLocal)
{
#if USE_INSTANCING
	half3x3 InstanceToWorld = mul(GetInstanceToLocal3x3(Intermediates), DFToFloat3x3(GetInstanceData(Intermediates).LocalToWorld));
	// remove scaling
	InstanceToWorld[0] = normalize(InstanceToWorld[0]);
	InstanceToWorld[1] = normalize(InstanceToWorld[1]);
	InstanceToWorld[2] = normalize(InstanceToWorld[2]);
	half3x3 TangentToWorld = mul(TangentToLocal, InstanceToWorld);
#else
	half3x3 TangentToWorld = CalcTangentToWorldNoScale(Intermediates, TangentToLocal);
#endif	// USE_INSTANCING
	return TangentToWorld;
}

FVertexFactoryIntermediatesCommon GetVertexFactoryCommonIntermediates(FSceneDataIntermediates SceneData, FVertexFactoryInstanceInput InstanceInput)
{
	FVertexFactoryIntermediatesCommon Intermediates = (FVertexFactoryIntermediatesCommon)0;
	Intermediates.SceneData = SceneData;

#if USE_INSTANCING
	Intermediates.InstanceInput = InstanceInput;
#endif

#if USE_SPLINEDEFORM
	Intermediates.SplineMeshParams = GetSplineMeshParams(GetInstanceData(Intermediates));
#endif

	return Intermediates;
}

#define LOCALVF_GET_COMMON_INTERMEDIATES(VFInput) GetVertexFactoryCommonIntermediates(VF_GPUSCENE_GET_INTERMEDIATES(VFInput), LOCALVF_GET_INSTANCE_INPUT(VFInput))

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	Intermediates.Common = LOCALVF_GET_COMMON_INTERMEDIATES(Input);

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Intermediates);
	FInstanceSceneData InstanceData = GetInstanceData(Intermediates);

#if USES_WORLD_POSITION_OFFSET
	#if ALWAYS_EVALUATE_WORLD_POSITION_OFFSET
		Intermediates.bEvaluateWorldPositionOffset = true;
	#else
		// Initialize this flag with culling results when applicable.
		Intermediates.bEvaluateWorldPositionOffset = (Intermediates.Common.SceneData.CullingFlags & INSTANCE_CULLING_FLAG_EVALUATE_WPO) != 0;
	#endif
#endif

	Intermediates.IsVisible = 1.0f;

#if VF_USE_PRIMITIVE_SCENE_DATA
	// Calculate the local (within the range of instance IDs belonging to a given primitive), as this can be used to load custom stuff
	Intermediates.PrimitiveLocalInstanceIndex = Intermediates.Common.SceneData.InstanceId - PrimitiveData.InstanceSceneDataOffset;

#if USE_INSTANCE_CULLING

#if USE_EDITOR_SHADERS
	Intermediates.HitProxyId = float4(InstanceData.EditorData.HitProxyId, 0.0f);
#else // !USE_EDITOR_SHADERS
	Intermediates.HitProxyId = float4(0.0f, 0.0f, 0.0f, 0.0f);
#endif // USE_EDITOR_SHADERS
	
	Intermediates.PerInstanceParams = float4(1.0f, Intermediates.IsVisible, 0.0f, 0.0f);
#if NEEDS_PER_INSTANCE_PARAMS
	{
		float3 InstanceTranslatedLocation = TransformLocalToTranslatedWorld(InstanceData.LocalBoundsCenter, InstanceData.LocalToWorld).xyz;
#if USES_PER_INSTANCE_FADE_AMOUNT
#if IS_INSTANCED_STATIC_MESH_VF
		// PerInstanceParams.x: per-instance fade out amount
		Intermediates.PerInstanceParams.x = 1.0 - saturate((length(InstanceTranslatedLocation) - InstancedVFLooseParameters.InstancingFadeOutParams.x) * InstancedVFLooseParameters.InstancingFadeOutParams.y);
#else
		Intermediates.PerInstanceParams.x = 1.0 - saturate((length(InstanceTranslatedLocation) - PrimitiveData.InstancingFadeOutParams.x) * PrimitiveData.InstancingFadeOutParams.y);
#endif
#endif
	#if USE_DITHERED_LOD_TRANSITION

#if !IS_INSTANCED_STATIC_MESH_VF
		// Note: in the old path (with a separate VF) we fetch the parameters from InstancedVFLooseParameters, but that is not possible with the shared VF
#define InstancedVFLooseParameters LocalVF
#endif
		if (InstancedVFLooseParameters.InstancingViewZCompareOne.w != 0.0f)
		{
			float RandomLOD = InstancedVFLooseParameters.InstancingViewZCompareZero.w * InstanceData.RandomID; 
			float ViewZZero = length(InstanceTranslatedLocation - InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginZero.xyz) + RandomLOD;
			float ViewZOne = length(InstanceTranslatedLocation - InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginOne.xyz) + RandomLOD;
			Intermediates.PerInstanceParams.z =
				dot(float3(ViewZZero.xxx > InstancedVFLooseParameters.InstancingViewZCompareZero.xyz), InstancedVFLooseParameters.InstancingViewZConstant.xyz) * InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginZero.w +
				dot(float3(ViewZOne.xxx > InstancedVFLooseParameters.InstancingViewZCompareOne.xyz), InstancedVFLooseParameters.InstancingViewZConstant.xyz) * InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginOne.w;
			Intermediates.PerInstanceParams.y *= abs(Intermediates.PerInstanceParams.z) < .999;
			Intermediates.IsVisible = Intermediates.PerInstanceParams.y;
		}
	#else
		Intermediates.PerInstanceParams.z = 0;
	#endif

		// Disable WPO if this instance is not visible for dithered LOD transition or instance fade reasons
		if (Intermediates.IsVisible == 0.0f)
		{
			Intermediates.bEvaluateWorldPositionOffset = false;
		}
	}
#endif // NEEDS_PER_INSTANCE_PARAMS

#endif // USE_INSTANCE_CULLING
#endif // VF_USE_PRIMITIVE_SCENE_DATA

#if MANUAL_VERTEX_FETCH
	Intermediates.Color = LocalVF.VertexFetch_ColorComponentsBuffer[(LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) & LocalVF.VertexFetch_Parameters[VF_ColorIndexMask_Index]] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
#else
	Intermediates.Color = Input.Color FCOLOR_COMPONENT_SWIZZLE; // Swizzle vertex color.
#endif

#if USE_INSTANCING && MANUAL_VERTEX_FETCH
	uint InstanceId = GetInstanceIdFromVF(Input);
	Intermediates.InstanceLightMapAndShadowMapUVBias = InstanceVF.VertexFetch_InstanceLightmapBuffer[(InstanceId + InstanceOffset)];
#elif USE_INSTANCING
	Intermediates.InstanceLightMapAndShadowMapUVBias = Input.InstanceLightMapAndShadowMapUVBias;
#endif

	float TangentSign = 1.0;
	Intermediates.TangentToLocal = CalcTangentToLocal(Input, Intermediates, TangentSign);
	Intermediates.TangentToWorld = CalcTangentToWorld(Intermediates, Intermediates.TangentToLocal);
	Intermediates.TangentToWorldSign = TangentSign * GetInstanceData(Intermediates).DeterminantSign;

#if USE_INSTANCING
	// GPUCULL_TODO: This can't be right
	float3 InstanceTranslatedLocation = TransformLocalToTranslatedWorld(GetInstanceOrigin(Intermediates), PrimitiveData.LocalToWorld).xyz;
	float InstanceViewDistance = length(InstanceTranslatedLocation);
	#if	NEEDS_PER_INSTANCE_PARAMS	
		// x = per-instance fade out factor, y = zero or one depending of if it is shown at all, z is dither cutoff 
		// PerInstanceParams.y stores a hide/show flag for this instance
		Intermediates.PerInstanceParams.x = 1.0 - saturate((InstanceViewDistance - InstancedVFLooseParameters.InstancingFadeOutParams.x) * InstancedVFLooseParameters.InstancingFadeOutParams.y);
		// InstancingFadeOutParams.z,w are RenderSelected and RenderDeselected respectively.
		#if USE_DITHERED_LOD_TRANSITION
			float RandomLOD = InstancedVFLooseParameters.InstancingViewZCompareZero.w * InstanceData.RandomID;
			float ViewZZero = length(InstanceTranslatedLocation - InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginZero.xyz) + RandomLOD;
			float ViewZOne = length(InstanceTranslatedLocation - InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginOne.xyz) + RandomLOD;
			Intermediates.PerInstanceParams.z = 
				dot(float3(ViewZZero.xxx > InstancedVFLooseParameters.InstancingViewZCompareZero.xyz), InstancedVFLooseParameters.InstancingViewZConstant.xyz) * InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginZero.w +
				dot(float3(ViewZOne.xxx > InstancedVFLooseParameters.InstancingViewZCompareOne.xyz), InstancedVFLooseParameters.InstancingViewZConstant.xyz) * InstancedVFLooseParameters.InstancingTranslatedWorldViewOriginOne.w;
			Intermediates.PerInstanceParams.y *= abs(Intermediates.PerInstanceParams.z) < .999;
		#else
			Intermediates.PerInstanceParams.z = 0;
		#endif
	
		#if USES_PER_INSTANCE_CUSTOM_DATA
			// index into instance CustomData
			Intermediates.PerInstanceParams.w = asfloat(GetInstanceId(GetInstanceIdFromVF(Input)) + InstanceOffset);
		#endif

		// Disable WPO if this instance is not visible for dithered LOD transition or instance fade reasons
		if (Intermediates.PerInstanceParams.y == 0.0f)
		{
			Intermediates.bEvaluateWorldPositionOffset = false;
		}
	#else
		Intermediates.PerInstanceParams = float4(1.f,Intermediates.IsVisible,0.f,0.f);
	#endif // NEEDS_PER_INSTANCE_PARAMS

	const float InstanceViewDistSq = Square(InstanceViewDistance);
#else
	// get the "instance" view distance from the primitive bounding center
	const float InstanceViewDistSq = length2(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation));
#endif // USE_INSTANCING

#if !USE_INSTANCE_CULLING && USES_WORLD_POSITION_OFFSET && !ALWAYS_EVALUATE_WORLD_POSITION_OFFSET && (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
	// In this case, we have to do the WPO disable distance check in the VS because it wasn't done in instance culling
	if (Intermediates.bEvaluateWorldPositionOffset)
	{
		const bool bPrimEvalWPO = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET) != 0;
		const bool bWPODisableDistance = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE) != 0;
		
		if (!bPrimEvalWPO || (bWPODisableDistance && InstanceViewDistSq >= PrimitiveData.InstanceWPODisableDistanceSquared))
		{
			Intermediates.bEvaluateWorldPositionOffset = false;
		}
	}
#endif

	if (IsGPUSkinPassThrough())
	{
	#if SUPPORT_GPUSKIN_PASSTHROUGH
	#if MANUAL_VERTEX_FETCH
		uint PreSkinVertexOffset = LocalVF.PreSkinBaseVertexIndex + Input.VertexId * 3;
		Intermediates.PreSkinPosition.x = LocalVF.VertexFetch_PreSkinPositionBuffer[PreSkinVertexOffset + 0];
		Intermediates.PreSkinPosition.y = LocalVF.VertexFetch_PreSkinPositionBuffer[PreSkinVertexOffset + 1];
		Intermediates.PreSkinPosition.z = LocalVF.VertexFetch_PreSkinPositionBuffer[PreSkinVertexOffset + 2];
	#else
		Intermediates.PreSkinPosition = Input.PreSkinPosition.xyz;
	#endif
	#endif
	}
	else
	{
		Intermediates.PreSkinPosition = Input.Position.xyz;
	}

	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
half3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.TangentToLocal;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;

	// Scale to zero if not visible, seems a bit wild but whatever
	float4 WorldPos = TransformLocalToTranslatedWorld(CalcPositionPrimitiveSpace(Intermediates.Common, Input.Position).xyz, LocalToWorld);
	return WorldPos * Intermediates.IsVisible;
}

// @return local position relative to primitive
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcPositionInstanceSpace(Intermediates.Common, Input.Position).xyz;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	// Really only the last two components of the packed UVs have the opportunity to be uninitialized
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

	FInstanceSceneData InstanceData = GetInstanceData(Intermediates);

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);
	
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}

#elif NUM_MATERIAL_TEXCOORDS_VERTEX == 0 && USE_PARTICLE_SUBUVS
	#if MANUAL_VERTEX_FETCH
		SetUV(Interpolants, 0, LocalVF.VertexFetch_TexCoordBuffer[LocalVF.VertexFetch_Parameters[VF_NumTexcoords_Index] * (LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId)]);
	#else
		SetUV(Interpolants, 0, Input.TexCoords0);
	#endif
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	float2 LightMapCoordinate = 0;
	float2 ShadowMapCoordinate = 0;
	#if MANUAL_VERTEX_FETCH
		float2 LightMapCoordinateInput = LocalVF.VertexFetch_TexCoordBuffer[LocalVF.VertexFetch_Parameters[VF_NumTexcoords_Index] * (LocalVF.VertexFetch_Parameters[VF_VertexOffset] + Input.VertexId) + LocalVF.VertexFetch_Parameters[FV_LightMapIndex_Index]];
	#else
		float2 LightMapCoordinateInput = Input.LightMapCoordinate;
	#endif

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(Intermediates);

	const bool bHasPerInstanceCoordinateScaleBias = (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS);

	uint LightmapDataIndex = PrimitiveData.LightmapDataIndex + LocalVF.LODLightmapDataIndex;
	uint LightmapUVIndex = PrimitiveData.LightmapUVIndex;
	float4 LightMapCoordinateScaleBias = VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(Input, LightmapDataIndex);
	#if USE_INSTANCE_CULLING
		const float2 InstanceLightMapScaleBias = CondMask(bHasPerInstanceCoordinateScaleBias, InstanceData.LightMapAndShadowMapUVBias.xy, LightMapCoordinateScaleBias.zw);
		LightMapCoordinate = LightMapCoordinateInput * LightMapCoordinateScaleBias.xy + InstanceLightMapScaleBias;
	#elif USE_INSTANCING
		LightMapCoordinate = LightMapCoordinateInput * LightMapCoordinateScaleBias.xy + GetInstanceLightMapBias(Intermediates);
	#else
		LightMapCoordinate = LightMapCoordinateInput * LightMapCoordinateScaleBias.xy + LightMapCoordinateScaleBias.zw;
	#endif
	#if STATICLIGHTING_TEXTUREMASK
		float4 ShadowMapCoordinateScaleBias = VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(Input, LightmapDataIndex);

		#if USE_INSTANCE_CULLING
			const float2 InstanceShadowMapScaleBias = CondMask(bHasPerInstanceCoordinateScaleBias, InstanceData.LightMapAndShadowMapUVBias.zw, ShadowMapCoordinateScaleBias.zw);
			ShadowMapCoordinate = LightMapCoordinateInput * ShadowMapCoordinateScaleBias.xy + InstanceShadowMapScaleBias;
		#elif USE_INSTANCING
			ShadowMapCoordinate = LightMapCoordinateInput * ShadowMapCoordinateScaleBias.xy + GetInstanceShadowMapBias(Intermediates);
		#else
			ShadowMapCoordinate = LightMapCoordinateInput * ShadowMapCoordinateScaleBias.xy + ShadowMapCoordinateScaleBias.zw;
		#endif
	#endif	// STATICLIGHTING_TEXTUREMASK

	SetLightMapCoordinate(Interpolants, LightMapCoordinate, ShadowMapCoordinate);
	SetLightmapDataIndex(Interpolants, LightmapDataIndex);
#endif	// NEEDS_LIGHTMAP_COORDINATE

	SetTangents(Interpolants, Intermediates.TangentToWorld[0], Intermediates.TangentToWorld[2], Intermediates.TangentToWorldSign);
	SetColor(Interpolants, Intermediates.Color);

#if	NEEDS_PER_INSTANCE_PARAMS
	Interpolants.PerInstanceParams = Intermediates.PerInstanceParams;
#endif

#if LOCAL_VF_INTERPOLATE_CUSTOM_DATA_OFFSET
	Interpolants.CustomDataOffset = InstanceData.CustomDataOffset;
	Interpolants.CustomDataCount  = InstanceData.CustomDataCount;
#endif

#if LOCAL_VF_INTERPOLATE_PER_INSTANCE_RANDOM
	#if USE_INSTANCING
		Interpolants.PerInstanceRandom = GetInstanceRandom(Intermediates);
	#else
		Interpolants.PerInstanceRandom = InstanceData.RandomID;
	#endif
#endif

	SetSceneData(Interpolants, Intermediates.Common.SceneData);
#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS
#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
	SetInstanceLocalToWorld(Interpolants, VertexParameters.InstanceLocalToWorld);
#endif
#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
	SetInstanceWorldToLocal(Interpolants, VertexParameters.InstanceWorldToLocal);
#endif
#endif
	return Interpolants;
}

/** X for depth-only pass */
float4 VertexFactoryGetWorldPosition(FPositionOnlyVertexFactoryInput Input)
{
	FVertexFactoryIntermediatesCommon Intermediates = LOCALVF_GET_COMMON_INTERMEDIATES(Input);
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;

	return TransformLocalToTranslatedWorld(CalcPositionPrimitiveSpace(Intermediates, Input.Position).xyz, LocalToWorld);
}

/** for depth-only pass (slope depth bias) */
float4 VertexFactoryGetWorldPosition(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FVertexFactoryIntermediatesCommon Intermediates = LOCALVF_GET_COMMON_INTERMEDIATES(Input);
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;

	return TransformLocalToTranslatedWorld(CalcPositionPrimitiveSpace(Intermediates, Input.Position).xyz, LocalToWorld);
}

float3 VertexFactoryGetWorldNormal(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FVertexFactoryIntermediatesCommon Intermediates = LOCALVF_GET_COMMON_INTERMEDIATES(Input);
	FDFMatrix LocalToWorld = GetInstanceData(Intermediates).LocalToWorld;
	float3 InvScale = GetInstanceData(Intermediates).InvNonUniformScale;
	float3 Normal = Input.Normal.xyz;

#if USE_INSTANCING
	const float3 InstanceTransformedNormal = mul(float4(Normal, 0), GetInstanceTransform(Input)).xyz;
	return RotateLocalToWorld(InstanceTransformedNormal, LocalToWorld, InvScale);
#else
	return RotateLocalToWorld(Normal, LocalToWorld, InvScale);
#endif
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToWorld[2];
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 PrevLocalPosition = Input.Position;

#if SUPPORT_GPUSKIN_PASSTHROUGH
	if (IsGPUSkinPassThrough())
	{
		// Duplicates logic in FGPUSkinVertexFactoryShaderParameters::GetElementShaderBindings().
		if (ResolvedView.FrameCounter == GPUSkinPassThroughVFLooseParameters.FrameNumber && ResolvedView.WorldIsPaused == 0)
		{
			uint Offset = Input.VertexId * 3;
			float3 PreviousPos;
			PreviousPos.x = GPUSkinPassThroughVFLooseParameters.PreviousPositionBuffer[Offset + 0];
			PreviousPos.y = GPUSkinPassThroughVFLooseParameters.PreviousPositionBuffer[Offset + 1];
			PreviousPos.z = GPUSkinPassThroughVFLooseParameters.PreviousPositionBuffer[Offset + 2];
			PrevLocalPosition = float4(PreviousPos, 1);	
		}
	}
	else
#endif // SUPPORT_GPUSKIN_PASSTHROUGH
	{
#if USE_SPLINEDEFORM && !RAYHITGROUPSHADER
		// Just like CalcWorldPosition, transform into mesh space
		PrevLocalPosition = float4(SplineMeshDeformLocalPos(Intermediates.Common.SplineMeshParams, Input.Position.xyz), Input.Position.w);
#endif
	}

	return PrevLocalPosition.xyz;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 PrevLocalPosition = float4(VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates), 1);
#if USE_INSTANCING
	float4x4 InstanceTransform = GetInstancePrevTransform(Intermediates);
	PrevLocalPosition = mul(Input.Position, InstanceTransform);
#endif
	FDFMatrix PreviousLocalToWorld = GetInstanceData(Intermediates).PrevLocalToWorld;

#if defined(IS_VELOCITY_SHADER) && STEREO_MOTION_VECTORS
	FLWCMatrix CurrentLocalToWorld = GetInstanceData(Intermediates).LocalToWorld;
	if (!IsSmoothRotation(PreviousLocalToWorld.M._11_21_31, PreviousLocalToWorld.M._12_22_32, CurrentLocalToWorld.M._11_21_31, CurrentLocalToWorld.M._12_22_32))
		PreviousLocalToWorld = CurrentLocalToWorld;
#endif

	return TransformPreviousLocalPositionToTranslatedWorld(PrevLocalPosition.xyz, PreviousLocalToWorld);
}

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	struct FVertexFactoryRayTracingInterpolants
	{
		FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	};

	float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
	{
	#if NUM_MATERIAL_TEXCOORDS
		return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
	#else // #if NUM_MATERIAL_TEXCOORDS
		return float2(0,0);
	#endif // #if NUM_MATERIAL_TEXCOORDS
	}

	FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
	{
		return Input.InterpolantsVSToPS;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
	{
		FVertexFactoryRayTracingInterpolants Interpolants;
		Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
		return Interpolants;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
	{
		// Default initialize. Otherwise, some graphics pipelines that
		// couple tessellation with geometry shaders won't write to all TEXCOORD semantics,
		// but read from them when <FVertexFactoryRayTracingInterpolants> is being copied as a whole.
		FVertexFactoryRayTracingInterpolants O = (FVertexFactoryRayTracingInterpolants)0;

#if VF_USE_PRIMITIVE_SCENE_DATA
		O.InterpolantsVSToPS.InstanceId = a.InterpolantsVSToPS.InstanceId;
#endif	

#if LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX
		O.InterpolantsVSToPS.LightmapDataIndex = a.InterpolantsVSToPS.LightmapDataIndex;
#endif

#if LOCAL_VF_INTERPOLATE_CUSTOM_DATA_OFFSET
		O.InterpolantsVSToPS.CustomDataOffset = a.InterpolantsVSToPS.CustomDataOffset;
		O.InterpolantsVSToPS.CustomDataCount  = a.InterpolantsVSToPS.CustomDataCount;
#endif

#if LOCAL_VF_INTERPOLATE_PER_INSTANCE_RANDOM
		O.InterpolantsVSToPS.PerInstanceRandom = a.InterpolantsVSToPS.PerInstanceRandom;
#endif

#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS
#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
		O.InterpolantsVSToPS.InstanceLocalToWorld = a.InterpolantsVSToPS.InstanceLocalToWorld;
#endif

#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
		O.InterpolantsVSToPS.InstanceWorldToLocal = a.InterpolantsVSToPS.InstanceWorldToLocal;
#endif
#endif


		// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
		// interpolated normal from 'whatever' interpolation scheme we're using
	
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
	#if INTERPOLATE_VERTEX_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.Color);
	#endif
	#if	NEEDS_PER_INSTANCE_PARAMS
		INTERPOLATE_MEMBER(InterpolantsVSToPS.PerInstanceParams);
	#endif

	#if NEEDS_LIGHTMAP_COORDINATE
		INTERPOLATE_MEMBER(InterpolantsVSToPS.LightMapCoordinate);
	#endif

	#if NUM_TEX_COORD_INTERPOLATORS
		UNROLL
		for(int tc = 0; tc < (NUM_TEX_COORD_INTERPOLATORS+1)/2; ++tc)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[tc]);
		}
	#elif USE_PARTICLE_SUBUVS
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[0]);
	#endif

		return O;
	}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(GetPrimitiveId(Interpolants));
	return float4(DFFastToTranslatedWorld(PrimitiveData.ObjectWorldPosition, ResolvedView.PreViewTranslation), PrimitiveData.ObjectRadius);
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return GetPrimitiveId(Interpolants);
}

FSceneDataIntermediates GetSceneDataIntermediates(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.Common.SceneData;
}

// We implemented GetSceneDataIntermediates ourselves
#define VF_IMPLEMENTED_GET_SCENE_DATA_INTERMEDIATES

#include "VertexFactoryDefaultInterface.ush"
