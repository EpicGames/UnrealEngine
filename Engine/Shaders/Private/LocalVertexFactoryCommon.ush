// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalVertexFactoryCommon.usf: Local vertex factory common functionality
=============================================================================*/

#ifndef IS_INSTANCED_STATIC_MESH_VF
	#define IS_INSTANCED_STATIC_MESH_VF 0
#endif

#define NEEDS_PER_INSTANCE_PARAMS ((USE_INSTANCING || USE_INSTANCE_CULLING) && (USES_PER_INSTANCE_FADE_AMOUNT || USES_SPEEDTREE || USE_DITHERED_LOD_TRANSITION || (!VF_USE_PRIMITIVE_SCENE_DATA && USES_PER_INSTANCE_CUSTOM_DATA)))

// Preserve behavior with separate ISM VF (primarily for mobile GPU-Scene platforms)
#define LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS (IS_INSTANCED_STATIC_MESH_VF || !(VF_USE_PRIMITIVE_SCENE_DATA))
#define LOCAL_VF_INTERPOLATE_PER_INSTANCE_RANDOM (IS_INSTANCED_STATIC_MESH_VF && NEEDS_PER_INSTANCE_RANDOM_PS)
#define LOCAL_VF_INTERPOLATE_CUSTOM_DATA_OFFSET  (IS_INSTANCED_STATIC_MESH_VF && USES_PER_INSTANCE_CUSTOM_DATA && VF_USE_PRIMITIVE_SCENE_DATA)
#define LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX (NEEDS_LIGHTMAP_COORDINATE && VF_USE_PRIMITIVE_SCENE_DATA)

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if INTERPOLATE_VERTEX_COLOR
	half4	Color : COLOR0;
#endif

#if NEEDS_PER_INSTANCE_PARAMS
	// x = per-instance fade out amount, y = hide/show flag, z dither fade cutoff, w - CustomData index
	nointerpolation float4  PerInstanceParams : COLOR1;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2]	: TEXCOORD0;
#elif USE_PARTICLE_SUBUVS
	float4	TexCoords[1] : TEXCOORD0;
#endif

#if NEEDS_LIGHTMAP_COORDINATE
	float4	LightMapCoordinate : TEXCOORD4;
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA
	nointerpolation uint InstanceId : INSTANCE_ID;
#endif

#if LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX
	nointerpolation uint LightmapDataIndex : LIGHTMAP_ID;
#endif

#if LOCAL_VF_INTERPOLATE_CUSTOM_DATA_OFFSET
	nointerpolation uint CustomDataOffset : CUSTOM_DATA_OFFSET;
	nointerpolation uint CustomDataCount : CUSTOM_DATA_COUNT;
#endif
#if LOCAL_VF_INTERPOLATE_PER_INSTANCE_RANDOM
	nointerpolation float PerInstanceRandom : PER_INSTANCE_RANDOM;
#endif

#if HAIR_STRAND_MESH_FACTORY
	nointerpolation uint HairControlPointId			: HAIR_PRIMITIVE_ID;	// Control point ID
	float2 HairPrimitiveUV							: HAIR_PRIMITIVE_UV;	// U: parameteric distance between the two surrounding control points. V: parametric distance along the width.
#endif
#if HAIR_CARD_MESH_FACTORY
	float2 HairPrimitiveUV							: HAIR_PRIMITIVE_UV;	// Atlas UV
	float2 HairPrimitiveRootUV						: HAIR_PRIMITIVE_ROOTUV;// UV at the root point of the cards
	float4 HairPrimitiveMaterial					: HAIR_PRIMITIVE_MATERIAL;
	float  HairPrimitiveLength						: HAIR_PRIMITIVE_LENGTH;
	nointerpolation float HairPrimitiveGroupIndex	: HAIR_PRIMITIVE_GROUPINDEX;
#endif

#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS
#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
	nointerpolation float3x4 InstanceLocalToWorld : INSTANCE_LOCAL_TO_WORLD;
#endif
#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
	nointerpolation float3x4 InstanceWorldToLocal : INSTANCE_WORLD_TO_LOCAL;
#endif
#endif
};

#if NUM_TEX_COORD_INTERPOLATORS || USE_PARTICLE_SUBUVS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

float4 GetColor(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if INTERPOLATE_VERTEX_COLOR
	return Interpolants.Color;
#else
	return 0;
#endif
}

void SetColor(inout FVertexFactoryInterpolantsVSToPS Interpolants, float4 InValue)
{
#if INTERPOLATE_VERTEX_COLOR
	Interpolants.Color = InValue;
#endif
}

#if NEEDS_LIGHTMAP_COORDINATE
void GetLightMapCoordinates(FVertexFactoryInterpolantsVSToPS Interpolants, out float2 LightmapUV0, out float2 LightmapUV1, out uint LightmapDataIndex)
{
	LightmapUV0 = Interpolants.LightMapCoordinate.xy * float2( 1, 0.5 );
	LightmapUV1 = LightmapUV0 + float2( 0, 0.5 );

#if LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX
	LightmapDataIndex = Interpolants.LightmapDataIndex;
#else
	LightmapDataIndex = 0;
#endif
}

void GetShadowMapCoordinate(FVertexFactoryInterpolantsVSToPS Interpolants, out float2 ShadowMapCoordinate, out uint LightmapDataIndex)
{
#if LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX
	LightmapDataIndex = Interpolants.LightmapDataIndex;
#else
	LightmapDataIndex = 0;
#endif
	ShadowMapCoordinate = Interpolants.LightMapCoordinate.zw;
}

void SetLightMapCoordinate(inout FVertexFactoryInterpolantsVSToPS Interpolants, float2 InLightMapCoordinate, float2 InShadowMapCoordinate)
{
	Interpolants.LightMapCoordinate.xy = InLightMapCoordinate;
	Interpolants.LightMapCoordinate.zw = InShadowMapCoordinate;
}
#endif

float4 GetTangentToWorld2(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return Interpolants.TangentToWorld2;
}

float4 GetTangentToWorld0(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return Interpolants.TangentToWorld0;
}

void SetTangents(inout FVertexFactoryInterpolantsVSToPS Interpolants, float3 InTangentToWorld0, float3 InTangentToWorld2, float InTangentToWorldSign)
{
	Interpolants.TangentToWorld0 = float4(InTangentToWorld0,0);
	Interpolants.TangentToWorld2 = float4(InTangentToWorld2,InTangentToWorldSign);
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Interpolants.TangentToWorld2_Center = Interpolants.TangentToWorld2;
#endif
}

uint GetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
#if SHADER_USES_PRIMITIVE_UBO
	// Note: SHADER_USES_PRIMITIVE_UBO pushes the DrawInstanceId through InstanceId
	return Interpolants.InstanceId;
#else
	FInstanceSceneData InstanceData = GetInstanceSceneData(Interpolants.InstanceId);
	return InstanceData.PrimitiveId;
#endif
#else
	return 0;
#endif
}

void SetSceneData(inout FVertexFactoryInterpolantsVSToPS Interpolants, FSceneDataIntermediates SceneData)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
#if SHADER_USES_PRIMITIVE_UBO
	// Note: SHADER_USES_PRIMITIVE_UBO pushes the DrawInstanceId through SceneData.PrimitiveId
	Interpolants.InstanceId = SceneData.PrimitiveId;
#else
	Interpolants.InstanceId = SceneData.InstanceId;
#endif
#endif
}

void SetLightmapDataIndex(inout FVertexFactoryInterpolantsVSToPS Interpolants, uint LightmapDataIndex)
{
#if LOCAL_VF_INTERPOLATE_LIGHTMAP_DATA_INDEX
	Interpolants.LightmapDataIndex = LightmapDataIndex;
#endif
}

#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS && HAS_INSTANCE_LOCAL_TO_WORLD_PS
void SetInstanceLocalToWorld(inout FVertexFactoryInterpolantsVSToPS Interpolants, FDFMatrix InstanceLocalToWorld)
{
	float4x4 InstanceLocalToTranslatedWorld = DFFastToTranslatedWorld(InstanceLocalToWorld, ResolvedView.PrevPreViewTranslation);
	float4x4 InstanceLocalToTranslatedWorldT = transpose(InstanceLocalToTranslatedWorld);
	Interpolants.InstanceLocalToWorld[0] = InstanceLocalToTranslatedWorldT[0];
	Interpolants.InstanceLocalToWorld[1] = InstanceLocalToTranslatedWorldT[1];
	Interpolants.InstanceLocalToWorld[2] = InstanceLocalToTranslatedWorldT[2];
}

FDFMatrix GetInstanceLocalToWorld(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	float4x4 InstanceLocalToTranslatedWorld = transpose(float4x4(Interpolants.InstanceLocalToWorld[0], Interpolants.InstanceLocalToWorld[1], Interpolants.InstanceLocalToWorld[2], float4(0.0f, 0.0f, 0.0f, 1.0f)));
	return DFMultiplyAffineTranslation(InstanceLocalToTranslatedWorld, DFNegate(ResolvedView.PrevPreViewTranslation));
}
#endif

#if LOCAL_VF_INTERPOLATE_INSTANCE_TRANSFORMS && HAS_INSTANCE_WORLD_TO_LOCAL_PS

void SetInstanceWorldToLocal(inout FVertexFactoryInterpolantsVSToPS Interpolants, FDFInverseMatrix InstanceWorldToLocal)
{
	float4x4 InstanceTranslatedWorldToLocal = DFFastToTranslatedWorld(InstanceWorldToLocal, ResolvedView.PrevPreViewTranslation);
	float4x4 InstanceTranslatedWorldToLocalT = transpose(InstanceTranslatedWorldToLocal);
	Interpolants.InstanceWorldToLocal[0] = InstanceTranslatedWorldToLocalT[0];
	Interpolants.InstanceWorldToLocal[1] = InstanceTranslatedWorldToLocalT[1];
	Interpolants.InstanceWorldToLocal[2] = InstanceTranslatedWorldToLocalT[2];
}

FDFInverseMatrix GetInstanceWorldToLocal(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	float4x4 InstanceTranslatedWorldToLocal = transpose(float4x4(Interpolants.InstanceWorldToLocal[0], Interpolants.InstanceWorldToLocal[1], Interpolants.InstanceWorldToLocal[2], float4(0.0f, 0.0f, 0.0f, 1.0f)));
	return DFMultiplyTranslationAffine(ResolvedView.PrevPreViewTranslation, InstanceTranslatedWorldToLocal);
}
#endif
