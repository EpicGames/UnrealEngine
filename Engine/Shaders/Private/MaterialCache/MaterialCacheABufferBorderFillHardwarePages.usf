// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/MaterialCacheDefinitions.h"

StructuredBuffer<FMaterialCachePageWriteData> PageWriteData;

RWTexture2DArray<float4> RWABuffer;

Texture2DArray<uint> R8Mask;

SamplerState TextureSampler;

uint BorderWidth;
uint ThreadCount;

bool FindNearestInfillMaskPixel(uint3 ABufferPhysicalLocation, out uint3 ABufferPhysicalLocationInFill)
{
	static const uint TriangularIndices[] = { 0, 1, 1, 2, 2, 2, 3, 3, 3, 3 };
	
	const uint SampleCount = BorderWidth * (BorderWidth + 1) >> 1;

	// A somewhat naive infilling, find the nearest sample and repeat its shading. While not entirely accurate, it does avoid a second pass.
	// TODO[MP]: Likely have to check against the page bounds, otherwise might pull in irrelevant triangles.
	for (uint i = 0; i < SampleCount; i++)
	{
		const uint TriIndex       = TriangularIndices[i];
		const uint TriSampleIndex = i - (TriIndex * (TriIndex + 1) >> 1);
		const int2 TriSample      = int2(TriSampleIndex, TriIndex - TriSampleIndex);

		const int2 OffsetSwizzles[] = {
			int2( TriSample.x + 1,  TriSample.y    ),
			int2( TriSample.x    , -TriSample.y - 1),
			int2(-TriSample.x - 1, -TriSample.y - 1),
			int2(-TriSample.x - 1,  TriSample.y    ),
		};

		UNROLL_N(4)
		for (uint j = 0; j < 4; j++)
		{
			if (R8Mask[ABufferPhysicalLocation + uint3(OffsetSwizzles[j], 0)])
			{
				ABufferPhysicalLocationInFill = ABufferPhysicalLocation + uint3(OffsetSwizzles[j], 0);
				return true;
			}
		}
	}

	return false;
}

[numthreads(8, 8, 1)]
void BorderFillHardwarePagesMain(uint3 DTid : SV_DispatchThreadID)
{
	if (any(DTid.xy >= ThreadCount.xx))
	{
		return;
	}

	const FMaterialCachePageWriteData Data = PageWriteData[DTid.z];
	const uint3 ABufferPhysicalLocation = Data.ABufferPhysicalPosition + uint3(DTid.xy, 0);
	
	// Already masked?
	if (R8Mask[ABufferPhysicalLocation])
	{
		return;
	}
	
	uint3 ABufferPhysicalLocationInFill;
	if (FindNearestInfillMaskPixel(ABufferPhysicalLocation, ABufferPhysicalLocationInFill))
	{
		RWABuffer[ABufferPhysicalLocation] = RWABuffer[ABufferPhysicalLocationInFill];
	}
}
