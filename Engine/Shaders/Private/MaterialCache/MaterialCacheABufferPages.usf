// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "/Engine/Private/BlockCompressionCommon.ush"
#include "/Engine/Private/BCCompressionCommon.ush"

StructuredBuffer<FMaterialCachePageWriteData> PageWriteData;

#if COMPRESS_MODE == BC_NONE
RWTexture2D<float4> RWVTLayerUncompressed;
#elif COMPRESS_MODE == BC1 || COMPRESS_MODE == BC4 // COMPRESS_MODE == BC_NONE
RWTexture2D<uint2>  RWVTLayerCompressed64;
#else // COMPRESS_MODE == BC1 || COMPRESS_MODE == BC4
RWTexture2D<uint4>  RWVTLayerCompressed128;
#endif
SamplerState        TextureSampler;

Texture2DArray<float4> ABuffer;

uint   BlockOrThreadCount;
float2 TexelSize;
uint   bYCoCg;
uint   bSRGB;

[numthreads(8, 8, 1)]
void WritePagesMain(uint3 DTid : SV_DispatchThreadID)
{
	uint2 BlockOrThreadIndex = DTid.xy;
	
	if (any(BlockOrThreadIndex.xy >= BlockOrThreadCount.xx))
	{
		return;
	}

	const FMaterialCachePageWriteData Data = PageWriteData[DTid.z];
	
#if COMPRESS_MODE == BC_NONE
	static const uint BlockSH = 0;
#else // COMPRESS_MODE == BC_NONE
	static const uint BlockSH = 2;
#endif // COMPRESS_MODE == BC_NONE
	
	const uint3  ABufferPhysicalLocation = Data.ABufferPhysicalPosition + uint3(BlockOrThreadIndex.xy << BlockSH, 0);
	const float3 ABufferPhysicalUV       = float3(ABufferPhysicalLocation.xy * TexelSize.xy + TexelSize.xy * 0.5f, ABufferPhysicalLocation.z);
	const uint2  VTPhysicalLocation      = (Data.VTPhysicalPosition >> BlockSH) + BlockOrThreadIndex.xy;

#if COMPRESS_MODE == BC_NONE
	// If uncompressed, just copy it
	// It does raise the question if we should somehow render it directly
	RWVTLayerUncompressed[VTPhysicalLocation] = ABuffer[ABufferPhysicalLocation];
#elif COMPRESS_MODE == BC1 // COMPRESS_MODE == BC_NONE
	float3 BlockRGB[16];
	ReadBlockRGB(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockRGB);

	if (bSRGB)
	{
		RWVTLayerCompressed64[VTPhysicalLocation] = CompressBC1Block_SRGB(BlockRGB);
	}
	else
	{
		RWVTLayerCompressed64[VTPhysicalLocation] = CompressBC1Block(BlockRGB);
	}
#elif COMPRESS_MODE == BC3 // COMPRESS_MODE == BC1
	float3 BlockRGB[16];
	ReadBlockRGB(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockRGB);

	// Branches aren't problematic
	if (bYCoCg)
	{
		RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC3BlockYCoCg(BlockRGB);
	}
	else
	{
		float BlockA[16];
		ReadBlockAlpha(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockA);
		
		if (bSRGB)
		{
			RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC3Block_SRGB(BlockRGB, BlockA);
		}
		else
		{
			RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC3Block(BlockRGB, BlockA);
		}
	}
#elif COMPRESS_MODE == BC4 // COMPRESS_MODE == BC3
	float BlockX[16];
	ReadBlockX(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockX);

	RWVTLayerCompressed64[VTPhysicalLocation] = CompressBC4Block(BlockX);
#elif COMPRESS_MODE == BC5 // COMPRESS_MODE == BC4
	float BlockX[16];
	float BlockY[16];
	ReadBlockXY(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockX, BlockY);

	RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC5Block(BlockX, BlockY);
#elif COMPRESS_MODE == BC6 // COMPRESS_MODE == BC5
	float3 BlockRGB[16];
	ReadBlockRGB(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockRGB);

	RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC6HBlock(BlockRGB);
#elif COMPRESS_MODE == BC7 // COMPRESS_MODE == BC6
	float3 BlockRGB[16];
	ReadBlockRGB(ABuffer, TextureSampler, ABufferPhysicalUV, TexelSize.xy, BlockRGB);

	RWVTLayerCompressed128[VTPhysicalLocation] = CompressBC7Block(BlockRGB);
#else // COMPRESS_MODE == BC7
	#error Unexpected compression mode
#endif // COMPRESS_MODE == BC7
}
