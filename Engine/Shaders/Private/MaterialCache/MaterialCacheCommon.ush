// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Private/MortonCode.ush"
#include "/Engine/Private/SceneData.ush"

struct FMaterialCacheDescriptor
{
	uint TagOffset;
	uint UVChannel;
};

float2 GetMaterialCacheUV(FMaterialCacheDescriptor Descriptor, float2 SourceUV)
{
	return SourceUV;
}

float4 GetMaterialCacheUnwrapClipPosition(FMaterialCacheDescriptor Descriptor, float2 TexCoord, float4 UnwrapMinAndInvSize, float4 PageOffsetAndInvSize = float4(0, 0, 1, 1))
{
	// Transform to logical coordinate
	TexCoord = GetMaterialCacheUV(Descriptor, TexCoord);
	
	// Remap coordinate
	TexCoord = (TexCoord - UnwrapMinAndInvSize.xy) * UnwrapMinAndInvSize.zw;

	// Viewport remapping
	// Nanite multi-view rendering doesn't normalize the NDC space to the view rect, it remains as is
	TexCoord = PageOffsetAndInvSize.xy + PageOffsetAndInvSize.zw * TexCoord;
	
	// To NDC
	return float4(
		TexCoord.x * 2.0f - 1.0f,
		1.0f - TexCoord.y * 2.0f,
		0.0f,
		1.0f
	);
}

FMaterialCacheBinData GetMaterialCacheShadingData(uint Index)
{
	// Workaround for static uniform buffer typing around structured buffers
	
	const uint StrideDW4 = 4; // sizeof(FMaterialCacheBinData) / sizeof(uint4);
	uint4 DW0 = MaterialCachePass.ShadingBinData[StrideDW4 * Index + 0];
	uint4 DW4 = MaterialCachePass.ShadingBinData[StrideDW4 * Index + 1];
	uint4 DW8 = MaterialCachePass.ShadingBinData[StrideDW4 * Index + 2];

	FMaterialCacheBinData Data = (FMaterialCacheBinData)0;
	Data.ABufferPhysicalPosition = DW0.xyz;
	Data.PrimitiveData = DW0.w;
	Data.UVMinAndThreadAdvance = asfloat(DW4);
	Data.UVMinAndInvSize = asfloat(DW8);
	return Data;
}

uint2 GetMaterialCacheCSShadePixel(uint3 DTid, uint4 TileParams, uint4 TileOrderingParams)
{
	// TODO[MP]: This is not as clean as it could be, also alternatives to consider.
	// But it does preserve the morton ordering within the aligned tiles, and,
	// on the unaligned tiles by the reducing the morton window.

	// Aligned?
	if (DTid.x < TileParams.y)
	{
		return MortonDecode(DTid.x);
	}

	// Unaligned offset
	DTid.x -= TileParams.y;

	// Intra-border tile
	uint2 Offset = MortonDecode(DTid.x & TileOrderingParams.x);

	// Axis wise offset (shr/shl can't be combined, relying on masking)
	uint2 AxisOffset = uint2(
		TileParams.x,
		(DTid.x >> TileOrderingParams.y) << TileOrderingParams.z
	);

	// This is ugly, can be improved
	bool bAxis = DTid.x >= TileParams.z;
	Offset += bAxis ? uint2(AxisOffset.y - TileParams.x, AxisOffset.x) : AxisOffset.xy;

	return Offset;
}

uint GetMaterialCacheUVCoordinateIndex(in FPrimitiveSceneData Data)
{
	return Data.MaterialCacheDescriptor.x >> 30;
}
