// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheCommon.ush"

#if MATERIAL_CACHE || MATERIAL_SAMPLES_CACHE || NUM_VIRTUALTEXTURE_SAMPLES > 0
#include "/Engine/Private/VirtualTextureCommon.ush"
#endif // NUM_VIRTUALTEXTURE_SAMPLES

bool GetMaterialCacheState()
{
#if MATERIAL_CACHE
	return true;
#else // MATERIAL_CACHE
	return false;
#endif // MATERIAL_CACHE
}

#if NUM_VIRTUALTEXTURE_SAMPLES > 0
VTPageTableUniform MaterialCacheUnpackPageTableUniform(uint2 PackedPageTableUniform)
{
	VTPageTableUniform Result;
	Result.UVScale = 1.f;
	Result.SizeInPages.x = PackedPageTableUniform.y & 0xfff;
	Result.SizeInPages.y = Result.SizeInPages.x;
	Result.MaxAnisoLog2 = 0;
	Result.XOffsetInPages = PackedPageTableUniform.x & 0xfff;
	Result.YOffsetInPages = (PackedPageTableUniform.x >> 12) & 0xfff;
	Result.vPageTableMipBias = (PackedPageTableUniform.x >> 24) & 0xf;
	Result.MipBiasGroup = 0;
	Result.MaxLevel = (PackedPageTableUniform.y >> 24) & 0xf;
	Result.AdaptiveLevelBias = 0;
	Result.AllocatedAdaptiveLevelBias = PackedPageTableUniform.y >> 28;
	Result.bPageTable32Bit = (PackedPageTableUniform.y >> 12) & 0x1;
	Result.ShiftedPageTableID = PackedPageTableUniform.x & 0xf0000000;
	return Result;
}
#endif // NUM_VIRTUALTEXTURE_SAMPLES > 0

FMaterialCacheDescriptor GetMaterialCacheTextureDescriptor(FPrimitiveSceneData PrimitiveData)
{
	FMaterialCacheDescriptor Out = (FMaterialCacheDescriptor)0;
	Out.TagOffset             = PrimitiveData.MaterialCacheDescriptor & 0xFFFF;
	// 16--30 free
	Out.UVChannel             = (PrimitiveData.MaterialCacheDescriptor >> 30);
	return Out;
}

uint GetMaterialCacheTagOffset(FPrimitiveSceneData PrimitiveData)
{
	return GetMaterialCacheTextureDescriptor(PrimitiveData).TagOffset;
}

bool GetMaterialCacheTextureDescriptorIsValid(FPrimitiveSceneData PrimitiveData)
{
	return GetMaterialCacheTagOffset(PrimitiveData) != ~0u;
}
