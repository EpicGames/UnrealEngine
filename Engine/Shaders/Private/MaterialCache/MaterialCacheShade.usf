// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Private/Common.ush"

#define SceneTexturesStruct MaterialCachePass.SceneTextures 
#define EyeAdaptationStruct MaterialCachePass

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "/Engine/Private/MortonCode.ush"
#include "/Engine/Public/RootConstants.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheShadeCommon.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheCommon.ush"

#ifndef NUM_TEX_COORD_INTERPOLATORS
#define NUM_TEX_COORD_INTERPOLATORS 1
#endif // NUM_TEX_COORD_INTERPOLATORS

#ifndef USE_ANALYTIC_DERIVATIVES
#define USE_ANALYTIC_DERIVATIVES 1
#endif // USE_ANALYTIC_DERIVATIVES

COMPILER_ALLOW_CS_DERIVATIVES

Buffer<uint> PageIndirections;

FMaterialPixelParameters GetMaterialPixelParameters(in FMaterialCacheBinData BinData, uint2 ThreadIndex)
{
	FMaterialPixelParameters Out = (FMaterialPixelParameters)0;
	Out.PrimitiveId = BinData.PrimitiveData.x;

	// Scale the coordinate by the current thread
#if NUM_TEX_COORD_INTERPOLATORS
	const uint   UVCoordinateIndex    = GetRootConstant2();
	const float2 CenterThreadPosition = ThreadIndex + 0.5f.xx;
	
	Out.TexCoords[UVCoordinateIndex] = BinData.UVMinAndThreadAdvance.xy + BinData.UVMinAndThreadAdvance.zw * CenterThreadPosition;

#if USE_ANALYTIC_DERIVATIVES
	// TODO: Assume the gradients across the entire page
	Out.TexCoords_DDX[0] = 0.0f;
	Out.TexCoords_DDY[0] = 0.0f;
#endif // USE_ANALYTIC_DERIVATIVES
#endif // NUM_TEX_COORD_INTERPOLATORS
	
	return Out;
}

FMaterialCacheABufferTag ShadeAttributes(in FMaterialCacheBinData BinData, uint2 ThreadIndex)
{
	ResolvedView = ResolveView();

	FVertexFactoryInterpolantsVSToPS Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
	Interpolants.PixelPos = ThreadIndex;

	float4 SvPosition                    = float4(ThreadIndex + 0.5f.xx, 0.0f, 1.0f);
	const float4 ScreenPosition          = SvPositionToResolvedScreenPosition(SvPosition);
	const float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(BinData, ThreadIndex);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, true, TranslatedWorldPosition, TranslatedWorldPosition);

	FMaterialCacheABufferTag ABuffer = GetMaterialABuffer(MaterialParameters, PixelMaterialInputs);
	return ABuffer;
}

[numthreads(64, 1, 1)]
void Main(uint3 DTid : SV_DispatchThreadID)
{
	const uint PageIndex = PageIndirections[GetRootConstant0() + DTid.y];

	const EMaterialCacheFlag    Flags   = (EMaterialCacheFlag)GetRootConstant1();
	const FMaterialCacheBinData BinData = GetMaterialCacheShadingData(PageIndex);
	
	const uint2 PagePixelPos = GetMaterialCacheCSShadePixel(DTid, MaterialCachePass.TileParams, MaterialCachePass.TileOrderingParams);

	// Shader against the current layer
	FMaterialCacheABufferTag Top = ShadeAttributes(BinData, PagePixelPos);
	if (Top.Clipped)
	{
		return;
	}

	StoreMaterialABufferPixel(Top, Flags, BinData.ABufferPhysicalPosition, PagePixelPos);
}
