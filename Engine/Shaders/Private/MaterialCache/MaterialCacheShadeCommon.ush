// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/MaterialCache/MaterialCacheCommon.ush"
#include "/Engine/Shared/MaterialCacheDefinitions.h"

#ifndef MATERIAL_CACHE_TAG_INDEX
#define MATERIAL_CACHE_TAG_INDEX 0
#endif // MATERIAL_CACHE_TAG_INDEX

#define TOKEN_CONCAT(A, B, C)   A ## B ## C
#define TOKEN_EXPAND(A, B, C)   TOKEN_CONCAT(A, B, C)
#define TAG_NAME(NAME)          TOKEN_EXPAND(NAME, MATERIAL_CACHE_TAG_INDEX, )
#define TAG_NAME_ANALYTIC(NAME) TOKEN_EXPAND(NAME, MATERIAL_CACHE_TAG_INDEX, _Analytic)

// Current tag in use
typedef TAG_NAME(FMaterialCacheABuffer_) FMaterialCacheABufferTag;

FMaterialCacheABufferTag GetMaterialABuffer(in FMaterialPixelParameters MaterialParameters, in FPixelMaterialInputs PixelMaterialInputs)
{
	// Handle opacity
#if MATERIALBLENDING_ANY_TRANSLUCENT
	bool bClipped = GetMaterialOpacity(PixelMaterialInputs) - 1e-2f < 0.0f;
#elif MATERIALBLENDING_MASKED // MATERIALBLENDING_ANY_TRANSLUCENT
	bool bClipped = GetMaterialMask(PixelMaterialInputs) < 0.0f;
#else // MATERIALBLENDING_MASKED
	bool bClipped = false;
#endif // MATERIALBLENDING_MASKED

	// If clipped, assume black
	if (bClipped)
	{
		// TODO: Temporarily driving weight through opacity, will be removed once the material generation is set up
		//FMaterialCacheABufferTag ABuffer = (FMaterialCacheABufferTag)0;
		//ABuffer.Clipped = true;
		//return ABuffer;
	}

#if NUM_MATERIAL_OUTPUTS_GETMATERIALCACHE > 0
	#if USE_ANALYTIC_DERIVATIVES
		return TAG_NAME_ANALYTIC(GetMaterialCache)(MaterialParameters);
	#else // USE_ANALYTIC_DERIVATIVES
		return TAG_NAME(GetMaterialCache)(MaterialParameters);
	#endif // USE_ANALYTIC_DERIVATIVES
#else // NUM_MATERIAL_OUTPUTS_GETMATERIALCACHE > 0
	return TAG_NAME(GetABufferFromMaterialInputs_)(PixelMaterialInputs);
#endif // NUM_MATERIAL_OUTPUTS_GETMATERIALCACHE > 0
}

FMaterialCacheABufferTag GetMaterialBottomABuffer(EMaterialCacheFlag Flags, uint3 ABufferPhysicalPosition, uint2 PagePixelPos)
{
	if (Flags & MatCache_DefaultBottomLayer)
	{
		return TAG_NAME(DefaultMaterialCacheABuffer_)();
	}
	else
	{
		return TAG_NAME(LoadMaterialCacheABufferPixel_)(ABufferPhysicalPosition, PagePixelPos);
	}
}

void StoreMaterialABufferPixel(FMaterialCacheABufferTag Top, EMaterialCacheFlag Flags, uint3 ABufferPhysicalPosition, uint2 PagePixelPos)
{
	FMaterialCacheABufferTag Bottom = GetMaterialBottomABuffer(Flags, ABufferPhysicalPosition, PagePixelPos);
	
	Top = TAG_NAME(BlendMaterialCacheFixedFunctionLerp_)(Bottom, Top, Top.Weight);

	TAG_NAME(StoreMaterialCacheABufferPixel_)(Top, ABufferPhysicalPosition, PagePixelPos);
}
