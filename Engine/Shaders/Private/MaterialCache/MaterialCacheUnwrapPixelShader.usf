// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Private/Common.ush"
#include "/Engine/Public/RootConstants.ush"
 
#define SceneTexturesStruct MaterialCachePass.SceneTextures 

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "/Engine/Generated/UniformBuffers/MaterialCachePass.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheShadeCommon.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheCommon.ush"

struct FMaterialCacheInterpolantsVSToPS
{

};

struct FMaterialCacheVSToPS
{
	float4 DualWorldPosition : DUAL_POSITION;
};

void Main(
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FMaterialCacheInterpolantsVSToPS PassInterpolants,
	in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position,
	in FMaterialCacheVSToPS Input
	OPTIONAL_IsFrontFace
)
{
	const uint PageIndex = GetRootConstant0();
	
	const EMaterialCacheFlag    Flags   = (EMaterialCacheFlag)GetRootConstant2();
	const FMaterialCacheBinData BinData = GetMaterialCacheShadingData(PageIndex);
	
	const uint2 PagePixelPos = min(uint2(SvPosition.xy), MaterialCachePass.TileParams.ww);
	
	ResolvedView = ResolveView();
	
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);

	const float4 ScreenPosition          = SvPositionToResolvedScreenPosition(SvPosition);
	const float3 TranslatedWorldPosition = MaterialParameters.WorldPosition_CamRelative;
	
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParametersEx(
		MaterialParameters,
		PixelMaterialInputs,
		SvPosition,
		ScreenPosition,
		bIsFrontFace,
		Input.DualWorldPosition.xyz,
		Input.DualWorldPosition.xyz
	);

	FMaterialCacheABufferTag Top = GetMaterialABuffer(MaterialParameters, PixelMaterialInputs);
	clip(Top.Clipped ? -1 : 1);

	StoreMaterialABufferPixel(Top, Flags, BinData.ABufferPhysicalPosition, PagePixelPos);
	
	MaterialCachePass.RWABuffer_R8Mask[BinData.ABufferPhysicalPosition + uint3(PagePixelPos, 0)] = 0xFF;
}
