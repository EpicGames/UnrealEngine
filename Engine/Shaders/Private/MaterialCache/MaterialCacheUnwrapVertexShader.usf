// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Shared/MaterialCacheDefinitions.h"
#include "/Engine/Private/Common.ush"
#include "/Engine/Public/RootConstants.ush"

#define SceneTexturesStruct MaterialCachePass.SceneTextures

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "/Engine/Generated/UniformBuffers/MaterialCachePass.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheCommon.ush"

struct FMaterialCacheInterpolantsVSToPS
{
	
};

struct FMaterialCacheVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	FMaterialCacheInterpolantsVSToPS  PassInterpolants;

#if SUPPORTS_VIEWPORT_FROM_VS
	uint PageIndex : SV_RenderTargetArrayIndex;
#endif // SUPPORTS_VIEWPORT_FROM_VS
	
	float4 Position : SV_POSITION;
	
	float4 DualWorldPosition : DUAL_POSITION;
};

void Main(FVertexFactoryInput Input, out FMaterialCacheVSToPS Output)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	
	const float4   WorldPositionExcludingWPO = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	const float3x3 TangentToLocal            = VertexFactoryGetTangentToLocal(Input, VFIntermediates);	
	const float4   WorldPosition             = WorldPositionExcludingWPO;
	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	const uint PageIndex        = GetRootConstant0();
	const uint ABufferPageIndex = GetRootConstant1();

	const FMaterialCacheBinData BinData = GetMaterialCacheShadingData(PageIndex);

#if SUPPORTS_VIEWPORT_FROM_VS
	Output.PageIndex = ABufferPageIndex;
#endif // SUPPORTS_VIEWPORT_FROM_VS
	
#if NUM_MATERIAL_TEXCOORDS_VERTEX > 0
	const uint   UVCoordinateIndex = GetRootConstant3();
	const float2 MaterialCacheUV   = VertexParameters.TexCoords[UVCoordinateIndex].xy;
#else // NUM_MATERIAL_TEXCOORDS_VERTEX > 0
	float2 MaterialCacheUV = 0.0f;
#endif // NUM_MATERIAL_TEXCOORDS_VERTEX > 0

	Output.Position = GetMaterialCacheUnwrapClipPosition(
		GetMaterialCacheTextureDescriptor(GetPrimitiveData(VertexParameters)),
		MaterialCacheUV,
		BinData.UVMinAndInvSize
	);

	Output.DualWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
}
