// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * MaterialTemplate.usf: Filled in by FHLSLMaterialTranslator::GetMaterialShaderCode for each material being compiled.
 */

#ifndef ENABLE_NEW_HLSL_GENERATOR
#define ENABLE_NEW_HLSL_GENERATOR 0
#endif

#if !MATERIAL_LWC_ENABLED
#define UE_DF_FORCE_FP32_OPS 1
#endif

#ifndef USE_VERTEX_FACTORY_FWD
#define USE_VERTEX_FACTORY_FWD 0
#endif

#if USE_VERTEX_FACTORY_FWD
#include "/Engine/Generated/VertexFactoryFwd.ush"
#endif

#include "/Engine/Private/WorldSpaceMath.ush"
#include "/Engine/Private/SceneTexturesCommon.ush"
#include "/Engine/Private/EyeAdaptationCommon.ush"
#include "/Engine/Private/Random.ush"
#include "/Engine/Private/SobolRandom.ush"
#include "/Engine/Private/MonteCarlo.ush"
#include "/Engine/Generated/UniformBuffers/Material.ush"
#include "/Engine/Private/DepthOfFieldCommon.ush"
#include "/Engine/Private/CircleDOFCommon.ush"
#include "/Engine/Private/DistanceField/GlobalDistanceFieldShared.ush"
#include "/Engine/Private/PhysicsFieldSampler.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/HairShadingCommon.ush"
#include "/Engine/Private/HairStrands/HairCardsAttributeCommon.ush"
#include "/Engine/Private/HairStrands/HairStrandsAttributeCommon.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/SparseVolumeTexture/SparseVolumeTextureCommon.ush"
#include "/Engine/Shared/EnvironmentComponentsFlags.h"
#include "/Engine/Private/MeshPaintTextureCommon.ush"
#include "/Engine/Private/ColorSpace.ush"
#include "/Engine/Private/VirtualTextureMaterial.ush"
#include "/Engine/Private/MaterialCache/MaterialCacheMaterialCommon.ush"
#include "/Engine/Private/BlueNoiseCommon.ush"

#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
#include "/Engine/Private/Quantization.ush"
#endif

// Reduce latency of Nanite material base pass pixel shaders caused by register pressure due to the calculation of WPO
#if PIXELSHADER && IS_NANITE_SHADING_PASS && IS_BASE_PASS
	COMPILER_FORCE_WAVE32_MODE
#endif

// Update this GUID to force all material shaders to recompile
// Merge conflicts on this line should be resolved by generating a new GUID
#pragma message("UESHADERMETADATA_VERSION E745A0A0-E5E2-4976-B407-F7C36FCE9880")

#if USES_SPEEDTREE
	#include "/Engine/Private/SpeedTreeCommon.ush"
#endif

//////////////////////////////////////////////////////////////////////////
//! Must match ESceneTextureId

#define PPI_SceneColor 0
#define PPI_SceneDepth 1
#define PPI_DiffuseColor 2
#define PPI_SpecularColor 3
#define PPI_SubsurfaceColor 4
#define PPI_BaseColor 5
#define PPI_Specular 6
#define PPI_Metallic 7
#define PPI_WorldNormal 8
#define PPI_SeparateTranslucency 9
#define PPI_Opacity 10
#define PPI_Roughness 11
#define PPI_MaterialAO 12
#define PPI_CustomDepth 13
#define PPI_PostProcessInput0 14
#define PPI_PostProcessInput1 15
#define PPI_PostProcessInput2 16
#define PPI_PostProcessInput3 17
#define PPI_PostProcessInput4 18
#define PPI_PostProcessInput5 19 // (UNUSED)
#define PPI_PostProcessInput6 20 // (UNUSED)
#define PPI_DecalMask 21
#define PPI_ShadingModelColor 22
#define PPI_ShadingModelID 23
#define PPI_AmbientOcclusion 24
#define PPI_CustomStencil 25
#define PPI_StoredBaseColor 26
#define PPI_StoredSpecular 27
#define PPI_Velocity 28
#define PPI_WorldTangent 29
#define PPI_Anisotropy 30
#define PPI_IsFirstPerson 31

#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
// Arbitrary internal value to fetch local variable propagated alpha stored in the SceneTextureLookup function
#define PPI_PropagatedAlpha 64
#endif

// Defines to remap PPI_UserSceneTexture0-6 to PPI_PostProcessInput0-6, generated via GenerateUserSceneTextureRemapDefines
%{user_scene_texture_remap}

//////////////////////////////////////////////////////////////////////////

#define NUM_MATERIAL_TEXCOORDS_VERTEX %{num_material_texcoords_vertex}
#define NUM_MATERIAL_TEXCOORDS %{num_material_texcoords}
#define NUM_CUSTOM_VERTEX_INTERPOLATORS %{num_custom_vertex_interpolators}
#define NUM_TEX_COORD_INTERPOLATORS %{num_tex_coord_interpolators}

// Vertex interpolators offsets definition
%{vertex_interpolators_offsets_definition}

#if NUM_VIRTUALTEXTURE_SAMPLES || LIGHTMAP_VT_ENABLED
	#include "/Engine/Private/VirtualTextureCommon.ush"
	#include "/Engine/Public/IndirectVirtualTextureCommon.ush"
#endif

#include "/Engine/Private/MaterialTexture.ush"

#ifdef MIN_MATERIAL_TEXCOORDS 
	#include "/Engine/Private/MinMaterialTexCoords.ush"
#endif

#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	#include "/Engine/Private/SkyAtmosphereCommon.ush"
#endif

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER || POST_PROCESS_MATERIAL // For WaterSampleSceneDepthWithoutWater()
// This is currently included for post processing materials even if the SceneDepthWithoutWater texture is not sampled.
// It might be worth it to add a switch to only include it if the texture is actually required by the material.
	#include "/Engine/Private/SingleLayerWaterCommon.ush"
#endif

#include "/Engine/Private/PaniniProjection.ush"
#include "/Engine/Private/DBufferNormalReprojection.ush"

#ifndef USE_INSTANCE_CULLING
	#define USE_INSTANCE_CULLING 0
#endif

#ifndef USE_STENCIL_LOD_DITHER
	#define USE_STENCIL_LOD_DITHER	USE_STENCIL_LOD_DITHER_DEFAULT
#endif

//Materials also have to opt in to these features.
#define USE_EDITOR_COMPOSITING (USE_EDITOR_SHADERS && EDITOR_PRIMITIVE_MATERIAL)

#define MATERIALBLENDING_ANY_TRANSLUCENT (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE || SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE || SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE || SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY)
 
#define IS_MATERIAL_TRANSLUCENT_AND_LIT  (MATERIALBLENDING_TRANSLUCENT)

#define IS_MESHPARTICLE_FACTORY (PARTICLE_MESH_FACTORY || NIAGARA_MESH_FACTORY)

#define HAS_INSTANCE_LOCAL_TO_WORLD_PS	(NEEDS_INSTANCE_LOCAL_TO_WORLD_PS && (USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS))
#define HAS_INSTANCE_WORLD_TO_LOCAL_PS	(NEEDS_INSTANCE_WORLD_TO_LOCAL_PS && (USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS))

#define IS_HAIR_FACTORY (HAIR_STRAND_MESH_FACTORY || HAIR_CARD_MESH_FACTORY)

#if IS_NANITE_PASS && !IS_NANITE_TRANSLUCENCY
	#define ALLOW_CLIP 0
	#define clip(x)
	#if COMPUTE_SHADED
		// Threads are remapped into 2x2 quad layout, so quad read x/y/d work for ddx/ddy
		// https://microsoft.github.io/DirectX-Specs/d3d/HLSL_SM_6_6_Derivatives.html
	#elif COMPUTESHADER
		#define ddx(x) 0
		#define ddy(x) 0
		#define fwidth(x) 0
	#endif
#else
	#define ALLOW_CLIP 1
#endif

// Material shader uses Front material or root node is converted to Substrate data.
#define TEMPLATE_USES_SUBSTRATE					(SUBSTRATE_ENABLED && MATERIAL_IS_SUBSTRATE)
// Material shader is NOT exported to Substrate data (no FrontMaterial or SubstrateTree data). It uses legacy GBuffer data.
#define TEMPLATE_USES_LEGACYGBUFFERDATA			(SUBSTRATE_ENABLED && !MATERIAL_IS_SUBSTRATE)

#define SUBSTRATE_OPAQUE_MATERIAL				(TEMPLATE_USES_SUBSTRATE && ((!SUBSTRATE_MATERIAL_EXPORT_EXECUTED && MATERIALBLENDING_ANY_TRANSLUCENT<=0) || SUBSTRATE_MATERIAL_EXPORT_FROM_OPAQUE))
#define SUBSTRATE_TRANSLUCENT_MATERIAL			(TEMPLATE_USES_SUBSTRATE && ((!SUBSTRATE_MATERIAL_EXPORT_EXECUTED && MATERIALBLENDING_ANY_TRANSLUCENT>0 ) || SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT))
#define SUBSTRATE_OPAQUE_DEFERRED				(SUBSTRATE_OPAQUE_MATERIAL && !FORWARD_SHADING)
#define SUBSTRATE_TRANSLUCENT_FORWARD			(SUBSTRATE_TRANSLUCENT_MATERIAL && !FORWARD_SHADING)
#define SUBSTRATE_FORWARD_SHADING				(FORWARD_SHADING && (SUBSTRATE_OPAQUE_MATERIAL || SUBSTRATE_TRANSLUCENT_MATERIAL))
#define SUBSTRATE_OUTPUT_ROUGH_REFRACTION		(SUBSTRATE_ENABLED && SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED && SUBSTRATE_MATERIAL_OUTPUT_OPAQUE_ROUGH_REFRACTIONS)

// Premultiplied alpha override is only enabled when alpha composite blending is used and the root node input is fed.
#define SUBSTRATE_USE_PREMULTALPHA_OVERRIDE	(SUBSTRATE_ENABLED && MATERIALBLENDING_ALPHACOMPOSITE && SUBSTRATE_PREMULTIPLIED_ALPHA_OPACITY_OVERRIDEN)

#if TEMPLATE_USES_SUBSTRATE
#define SUBSTRATE_INLINE_SHADING 1
#endif

#if SUBSTRATE_TRANSLUCENT_FORWARD && !SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT && !defined(SUBSTRATE_DEFERRED_SHADING)
// This is for translucent surface to be able to fetch opaque material data such as normal, albedo and more from SceneTextures (See SceneTextureLookup).
#define SUBSTRATE_DEFERRED_SHADING 1
#endif

#if SUBSTRATE_ENABLED

#include "/Engine/Private/Substrate/Substrate.ush"

#ifdef SubstrateStruct
	#define SubstratePublicStruct SubstrateStruct
#else
	#define SubstratePublicStruct Substrate
#endif

#ifndef ALLOW_SUBSTRATE_PUBLIC_ACCESS
#define ALLOW_SUBSTRATE_PUBLIC_ACCESS 0
#endif

#if !SUBSTRATE_OPAQUE_DEFERRED || ALLOW_SUBSTRATE_PUBLIC_ACCESS
#include "/Engine/Public/SubstratePublic.ush"
#endif

#undef SubstratePublicStruct

#else
struct FSubstrateData
{
	uint Dummy;
};
FSubstrateData GetInitialisedSubstrateData() { return (FSubstrateData)0; }
#endif

#if TEMPLATE_USES_SUBSTRATE
#include "/Engine/Private/Substrate/SubstrateTree.ush"
#include "/Engine/Private/Substrate/SubstrateLegacyConversion.ush"	// This can only included here due to interaction between SUBSTRATE_MATERIAL_EXPORT_EXECUTED and SUBSTRATE_LEGACY_MATERIAL_APPLIES_FINAL_WEIGHT
#endif

#include "/Engine/Private/DBufferDecalShared.ush"

/**
 * Parameters used by vertex and pixel shaders to access particle properties.
 */
struct FMaterialParticleParameters
{
	/** Relative time [0-1]. */
	half RelativeTime;
	/** Fade amount due to motion blur. */
	half MotionBlurFade;
	/** Random value per particle [0-1]. */
	half Random;
	/** XYZ: Direction, W: Speed. */
	half4 Velocity;
	/** Per-particle color. */
	half4 Color;
	/** Particle translated world space position and size(radius). */
	float4 TranslatedWorldPositionAndSize;
	/** Previous Particle translated world space position and size */
	float4 PrevTranslatedWorldPositionAndSize;
	/** Particle world space position. */
	FWSVector3 WorldPosition;
	/** Previous particle world space position. */
	FWSVector3 PrevWorldPosition;
	/** Macro UV scale and bias. */
	half4 MacroUV;

	/** Dynamic parameters used by particle systems. */
#if NIAGARA_PARTICLE_FACTORY && (DYNAMIC_PARAMETERS_MASK != 0)
	uint DynamicParameterValidMask;
#endif
	half4 DynamicParameter;

#if( DYNAMIC_PARAMETERS_MASK & 2)
	half4 DynamicParameter1;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 4)
	half4 DynamicParameter2;
#endif

#if (DYNAMIC_PARAMETERS_MASK & 8)
	half4 DynamicParameter3;
#endif

	/** mesh particle transform */
	FDFMatrix ParticleToWorld;

	/** Inverse mesh particle transform */
	FDFInverseMatrix WorldToParticle;

#if USE_PARTICLE_SUBUVS
	/** SubUV texture coordinates*/
	MaterialFloat2 SubUVCoords[2];
	/** SubUV interpolation value*/
	MaterialFloat SubUVLerp;
#endif

	/** The size of the particle. */
	float2 Size;

	/** The sprite rotation */
	float SpriteRotation;
};

float4 GetDynamicParameter(FMaterialParticleParameters Parameters, float4 Default, int ParameterIndex=0)
{
#if (NIAGARA_PARTICLE_FACTORY)
	switch ( ParameterIndex )
	{
	#if (DYNAMIC_PARAMETERS_MASK & 1)
		case 0:
			return float4(
				(Parameters.DynamicParameterValidMask & 0x0001) == 0 ? Default.x : Parameters.DynamicParameter.x,
				(Parameters.DynamicParameterValidMask & 0x0002) == 0 ? Default.y : Parameters.DynamicParameter.y,
				(Parameters.DynamicParameterValidMask & 0x0004) == 0 ? Default.z : Parameters.DynamicParameter.z,
				(Parameters.DynamicParameterValidMask & 0x0008) == 0 ? Default.w : Parameters.DynamicParameter.w
			);
	#endif
	#if (DYNAMIC_PARAMETERS_MASK & 2)
		case 1:
			return float4(
				(Parameters.DynamicParameterValidMask & 0x0010) == 0 ? Default.x : Parameters.DynamicParameter1.x,
				(Parameters.DynamicParameterValidMask & 0x0020) == 0 ? Default.y : Parameters.DynamicParameter1.y,
				(Parameters.DynamicParameterValidMask & 0x0040) == 0 ? Default.z : Parameters.DynamicParameter1.z,
				(Parameters.DynamicParameterValidMask & 0x0080) == 0 ? Default.w : Parameters.DynamicParameter1.w
			);
	#endif
	#if (DYNAMIC_PARAMETERS_MASK & 4)
		case 2:
			return float4(
				(Parameters.DynamicParameterValidMask & 0x0100) == 0 ? Default.x : Parameters.DynamicParameter2.x,
				(Parameters.DynamicParameterValidMask & 0x0200) == 0 ? Default.y : Parameters.DynamicParameter2.y,
				(Parameters.DynamicParameterValidMask & 0x0400) == 0 ? Default.z : Parameters.DynamicParameter2.z,
				(Parameters.DynamicParameterValidMask & 0x0800) == 0 ? Default.w : Parameters.DynamicParameter2.w
			);
	#endif	
	#if (DYNAMIC_PARAMETERS_MASK & 8)
		case 3:
			return float4(
				(Parameters.DynamicParameterValidMask & 0x1000) == 0 ? Default.x : Parameters.DynamicParameter3.x,
				(Parameters.DynamicParameterValidMask & 0x2000) == 0 ? Default.y : Parameters.DynamicParameter3.y,
				(Parameters.DynamicParameterValidMask & 0x4000) == 0 ? Default.z : Parameters.DynamicParameter3.z,
				(Parameters.DynamicParameterValidMask & 0x8000) == 0 ? Default.w : Parameters.DynamicParameter3.w
			);
	#endif
		default:
			return Default;
	}
#elif (PARTICLE_FACTORY)
	if ( ParameterIndex == 0 )
	{
		return Parameters.DynamicParameter;
	}
#endif
	return Default;

}

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

/** Material declarations */
%{material_declarations}

/** FMaterialAttributes utilities */
%{material_attributes_utilities}
%{material_aggregates_declarations}

SHADER_POP_WARNINGS_STATE

/** 
 * Parameters calculated from the pixel material inputs.
 */
struct FPixelMaterialInputs
{
%{pixel_material_inputs}

#if TEMPLATE_USES_SUBSTRATE
FSubstrateData GetFrontSubstrateData()
{
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1
	return FullySimplifiedFrontMaterial;
#else
	return FrontMaterial;
#endif
}
#endif // TEMPLATE_USES_SUBSTRATE
};

/**
* Worldspace transforms/vectors, converted to the LWC format used by the material.
*/
struct FMaterialLWCData
{
	// Vertex/pixel
	FWSVector3 AbsoluteWorldPosition;
	FWSVector3 WorldPosition_NoOffsets;

	// Primitive
	FWSMatrix	LocalToWorld;
	FWSInverseMatrix WorldToLocal;
	FWSMatrix	PreviousLocalToWorld;
	FWSInverseMatrix PreviousWorldToLocal;

	FWSMatrix	InstanceToWorld;
	FWSInverseMatrix WorldToInstance;
	FWSMatrix	PreviousInstanceToWorld;

	FWSVector3	ObjectWorldPosition;
	FWSVector3	ActorWorldPosition;

	FWSMatrix ParticleToWorld;
	FWSInverseMatrix WorldToParticle;
	FWSVector3 ParticleWorldPosition;
	FWSVector3 PrevParticleWorldPosition;

	// View
	FWSVector3 PreViewTranslation;
	FWSVector3 PrevPreViewTranslation;
	FWSVector3 WorldViewOrigin;
	FWSVector3 PrevWorldViewOrigin;
	FWSVector3 WorldCameraOrigin;
	FWSVector3 PrevWorldCameraOrigin;
};

/** 
 * Parameters needed by pixel shader material inputs, related to Geometry.
 * These are independent of vertex factory.
 */
struct FMaterialPixelParameters
{
#if NUM_TEX_COORD_INTERPOLATORS
	float2 TexCoords[NUM_TEX_COORD_INTERPOLATORS];
#endif

	/** Interpolated vertex color, in linear color space. */
	half4 VertexColor;

	/** Normalized world space normal. */
	half3 WorldNormal;
	half3 NormalDistribution;
	
	/** Normalized world space tangent. */
	half3 WorldTangent;

	/** Normalized world space reflected camera vector. */
	half3 ReflectionVector;

	/** Normalized world space camera vector, which is the vector from the point being shaded to the camera position. */
	float3 CameraVector;

	/** World space light vector, only valid when rendering a light function. */
	half3 LightVector;

	/**
	 * Like SV_Position (.xy is pixel position at pixel center, z:DeviceZ, .w:SceneDepth)
	 * using shader generated value SV_POSITION
	 * Note: this is not relative to the current viewport.  RelativePixelPosition = MaterialParameters.SvPosition.xy - View.ViewRectMin.xy;
	 */
	float4 SvPosition;
		
	/** Post projection position reconstructed from SvPosition, before the divide by W. left..top -1..1, bottom..top -1..1  within the viewport, W is the SceneDepth */
	float4 ScreenPosition;

	/**
	 * The pixel UV for a view, considering the buffer resolution containing it.
	 * This is precomputed because when PixelDepthOffset is used, it affects ScreenPosition which would then prevent us from recovering the correct UV later. 
	 * See ApplyPixelDepthOffsetToMaterialParameters for the details */
	float2 ViewBufferUV;

#if IS_NANITE_PASS
	float4 PrevScreenPosition;
#endif

	half UnMirrored;

	half TwoSidedSign;

	/**
	 * Orthonormal rotation-only transform from tangent space to world space
	 * The transpose(TangentToWorld) is WorldToTangent, and TangentToWorld[2] is WorldVertexNormal
	 */
	half3x3 TangentToWorld;

#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	/** World vertex normal interpolated at the pixel center that is safe to use for derivatives. */
	half3 WorldVertexNormal_Center;
#endif

	/** 
	 * Interpolated worldspace position of this pixel
	 * todo: Make this TranslatedWorldPosition and also rename the VS WorldPosition to be TranslatedWorldPosition
	 */
	FDFVector3 AbsoluteWorldPosition;

	/** 
	 * Interpolated worldspace position of this pixel, centered around the camera
	 */
	float3 WorldPosition_CamRelative;

	/** 
	 * Interpolated worldspace position of this pixel, not including any world position offset or displacement.
	 * Only valid if shader is compiled with NEEDS_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS, otherwise just contains 0
	 */
	FDFVector3 WorldPosition_NoOffsets;

	/** 
	 * Interpolated worldspace position of this pixel, not including any world position offset or displacement.
	 * Only valid if shader is compiled with NEEDS_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS, otherwise just contains 0
	 */
	float3 WorldPosition_NoOffsets_CamRelative;

	/** Offset applied to the lighting position for translucency, used to break up aliasing artifacts. */
	half3 LightingPositionOffset;

	/** Derivatives */
	float3 WorldPosition_DDX;
	float3 WorldPosition_DDY;
	float3 WorldGeoNormal_DDX;
	float3 WorldGeoNormal_DDY;
	float4 VertexColor_DDX;
	float4 VertexColor_DDY;
	float4 ScreenPosition_DDX;
	float4 ScreenPosition_DDY;
	
#if NUM_TEX_COORD_INTERPOLATORS
	float2 TexCoords_DDX[NUM_TEX_COORD_INTERPOLATORS];
	float2 TexCoords_DDY[NUM_TEX_COORD_INTERPOLATORS];
#endif

	float AOMaterialMask;

#if LIGHTMAP_UV_ACCESS
	float2	LightmapUVs;
	float2	LightmapUVs_DDX;
	float2	LightmapUVs_DDY;
#endif

	half PerInstanceRandom;

#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_NANITE_PASS
	float4 PerInstanceParams;
#endif

	// Index into View.PrimitiveSceneData
	uint PrimitiveId;

#if IS_NANITE_PASS
	uint InstanceId;
#endif
#if MOBILE_MULTI_VIEW
	uint ViewId;
#endif

#if USES_PER_INSTANCE_CUSTOM_DATA && VF_USE_PRIMITIVE_SCENE_DATA
	uint CustomDataOffset;
	uint CustomDataCount;
#endif

	// Actual primitive Id
#if HAIR_STRAND_MESH_FACTORY
	uint	HairControlPointId;		// Control point ID
	float2	HairPrimitiveUV;		// U: parametric distance between the two surrounding control point. V: parametric distance along hair width
#endif
#if HAIR_CARD_MESH_FACTORY
	float2	HairPrimitiveUV;		// AtlasUV
	float2  HairPrimitiveRootUV;	// RootUV
	half4	HairPrimitiveMaterial;	// Card material
	half    HairPrimitiveLength;	// Card length
	half	HairPrimitiveGroupIndex;// Card group index
#endif

#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
	FDFMatrix InstanceLocalToWorld;
#endif
#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
	FDFInverseMatrix InstanceWorldToLocal;
#endif
	/** Per-particle properties. Only valid for particle vertex factories. */
	FMaterialParticleParameters Particle;

#if TEX_COORD_SCALE_ANALYSIS
	/** Parameters used by the MaterialTexCoordScales shader. */
	FTexCoordScalesParams TexCoordScalesParams;
#endif

#if COMPILER_HLSL
	// Workaround for "error X3067: 'GetObjectWorldPosition': ambiguous function call"
	// Which happens when FMaterialPixelParameters and FMaterialVertexParameters have the same number of floats with the HLSL compiler ver 9.29.952.3111
	// Function overload resolution appears to identify types based on how many floats / ints / etc they contain
	uint Dummy;
#endif

#if NUM_VIRTUALTEXTURE_SAMPLES || LIGHTMAP_VT_ENABLED
	FVirtualTextureFeedbackParams VirtualTextureFeedback;
#endif

#if WATER_MESH_FACTORY
	uint WaterWaveParamIndex;
#endif

#if SHADER_TYPE == ST_GrayscaleFont || SHADER_TYPE == ST_ColorFont || SHADER_TYPE == ST_SdfFont || SHADER_TYPE == ST_MsdfFont || SHADER_TYPE == ST_Dynamic
	float4 FontSignedDistanceData;
#endif

#if CLOUD_LAYER_PIXEL_SHADER
	float CloudSampleAltitude;
	float CloudSampleAltitudeInLayer;
	float CloudSampleNormAltitudeInLayer;
	float4 VolumeSampleConservativeDensity;
	float ShadowSampleDistance;

	float3 CloudEmptySpaceSkippingSphereCenterWorldPosition;
	float  CloudEmptySpaceSkippingSphereRadius;
#endif

#if TEMPLATE_USES_SUBSTRATE
	FSharedLocalBases SharedLocalBases;
	FSubstrateTree SubstrateTree;

#if (SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1) || ENABLE_NEW_HLSL_GENERATOR
	// The new translator always add those members in. This is because we do not hide fully simplified instructions behind a "#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1" for now.
	FSharedLocalBases SharedLocalBasesFullySimplified;
	FSubstrateTree SubstrateTreeFullySimplified;
#endif

	FSubstratePixelFootprint SubstratePixelFootprint;

	FSubstratePixelHeader GetFrontSubstrateHeader()
	{
		FSubstratePixelHeader SubstratePixelHeader = InitialiseSubstratePixelHeader();
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1
		SubstratePixelHeader.SubstrateTree = SubstrateTreeFullySimplified;
		SubstratePixelHeader.ClosureCount = SubstrateTreeFullySimplified.BSDFCount;
		SubstratePixelHeader.SharedLocalBases = SharedLocalBasesFullySimplified;
#else
		SubstratePixelHeader.SubstrateTree = SubstrateTree;
		SubstratePixelHeader.ClosureCount = SubstrateTree.BSDFCount;
		SubstratePixelHeader.SharedLocalBases = SharedLocalBases;
#endif
		SubstratePixelHeader.IrradianceAO = InitIrradianceAndOcclusion();
		return SubstratePixelHeader;
	}
#endif

	FMaterialAttributes MaterialAttributes;

	FMaterialLWCData LWCData;

%{material_pixel_parameter_decls}
};

%{material_cache_types}

/**
 * Compile out calls to StoreTexCoordScale if we're not doing texcoord scale analysis
 */
#if TEX_COORD_SCALE_ANALYSIS
#define MaterialStoreTexCoordScale(Parameters, UV, TextureReferenceIndex) StoreTexCoordScale(Parameters.TexCoordScalesParams, UV, TextureReferenceIndex)
#define MaterialStoreTexSample(Parameters, UV, TextureReferenceIndex) StoreTexSample(Parameters.TexCoordScalesParams, UV, TextureReferenceIndex)
#else
#define MaterialStoreTexCoordScale(Parameters, UV, TextureReferenceIndex) 1.0f
#define MaterialStoreTexSample(Parameters, UV, TextureReferenceIndex) 1.0f
#endif

// @todo compat hack
FMaterialPixelParameters MakeInitializedMaterialPixelParameters()
{
	FMaterialPixelParameters MPP;
	MPP = (FMaterialPixelParameters)0;
	MPP.TangentToWorld = float3x3(1,0,0,0,1,0,0,0,1);
	MPP.Particle.Color = half4(1,1,1,1);
	return MPP;
}

/** 
 * Parameters needed by vertex shader material inputs.
 * These are independent of vertex factory.
 */
struct FMaterialVertexParameters
{
	// Position in the translated world (VertexFactoryGetWorldPosition).
	// Previous position in the translated world (VertexFactoryGetPreviousWorldPosition) if
	//    computing material's output for previous frame (See {BasePassVertex,Velocity}Shader.usf).
	float3 WorldPosition;
	
	// Local position relative to the primitive, i.e. before transformation into worldspace. 
	float3 PositionPrimitiveSpace;
	// Local position relative to the instance. If instancing is disabled, this is the same as PositionPrimitiveSpace.
	float3 PositionInstanceSpace;

	// TangentToWorld[2] is WorldVertexNormal
	half3x3 TangentToWorld;

#if USES_PER_INSTANCE_CUSTOM_DATA && VF_USE_PRIMITIVE_SCENE_DATA
	uint CustomDataOffset;
	uint CustomDataCount;
#endif

	float PerInstanceRandom;
	
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS
	FDFMatrix InstanceLocalToWorld;
	FDFInverseMatrix InstanceWorldToLocal;
#endif
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_NANITE_PASS
	/** Per-instance properties. */
	float4 PerInstanceParams;
#if !USE_INSTANCE_CULLING
	uint InstanceId;
#endif
	uint InstanceOffset;
#endif
	// If either USE_INSTANCING or (IS_MESHPARTICLE_FACTORY && FEATURE_LEVEL >= FEATURE_LEVEL_SM4)
	// is true, PrevFrameLocalToWorld is a per-instance transform
	FDFMatrix PrevFrameLocalToWorld;

	float3 PreSkinnedPosition;
	float3 PreSkinnedNormal;

	half4 VertexColor;

#if ENABLE_NEW_HLSL_GENERATOR && NUM_TEX_COORD_INTERPOLATORS
	// NUM_MATERIAL_TEXCOORDS_VERTEX means the number of actual UV TexCoords used by the shader
	// NUM_TEX_COORD_INTERPOLATORS instead is the total interpolator registers to use: this is the value above plus all the user interpolators expressions in the material
	// Using NUM_TEX_COORD_INTERPOLATORS here so that we have a single array ot hold all interpolated data (UVs plus user).
	// Note: FMaterialVertex/PixelParameters structs are "local" to the material
	// Note: The code that maps to these registers is generated by ComputeUserVertexInterpolatorAllocation() in the new translator
	float2 TexCoords[NUM_TEX_COORD_INTERPOLATORS];
#elif NUM_MATERIAL_TEXCOORDS_VERTEX
	float2 TexCoords[NUM_MATERIAL_TEXCOORDS_VERTEX];
#endif

	/** Per-particle properties. Only valid for particle vertex factories. */
	FMaterialParticleParameters Particle;

#if WATER_MESH_FACTORY
	uint WaterWaveParamIndex;
#endif

	FMaterialAttributes MaterialAttributes;

	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;

	// FIXME: just for compatibility with assets that use custom HLSL expressions, will be removed once we fix up all these assets
	// Index into View.PrimitiveSceneData
	uint PrimitiveId;

	bool bEvaluateWorldPositionOffset;

	FMaterialLWCData LWCData;

	MaterialFloat3 WorldPositionOffset;
	
%{material_vertex_parameter_decls}
};

FMaterialVertexParameters MakeInitializedMaterialVertexParameters()
{
	FMaterialVertexParameters Result = (FMaterialVertexParameters)0;
	Result.PrimitiveId = INVALID_PRIMITIVE_ID;
	Result.bEvaluateWorldPositionOffset = true;
	Result.Particle.Color = half4(1,1,1,1);

	return Result;
}

#if GET_PRIMITIVE_DATA_OVERRIDE
	FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters);
	FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters);
#else
FPrimitiveSceneData GetPrimitiveData(FMaterialVertexParameters Parameters)
{
#if HAS_PRIMITIVE_UNIFORM_BUFFER
	return Parameters.SceneData.Primitive;
#else
	return (FPrimitiveSceneData)0;
#endif	
}

FPrimitiveSceneData GetPrimitiveData(FMaterialPixelParameters Parameters)
{
#if HAS_PRIMITIVE_UNIFORM_BUFFER
	return GetPrimitiveData(Parameters.PrimitiveId);
#else
	return (FPrimitiveSceneData)0;
#endif	
}
#endif 

bool UnpackUniform_bool(uint Packed, uint BitOffset)
{
	return (bool)((Packed >> BitOffset) & 0x1);
}

bool2 UnpackUniform_bool2(uint Packed, uint BitOffset)
{
	return bool2(UnpackUniform_bool(Packed, BitOffset), UnpackUniform_bool(Packed, BitOffset + 1));
}

bool3 UnpackUniform_bool3(uint Packed, uint BitOffset)
{
	return bool3(UnpackUniform_bool(Packed, BitOffset), UnpackUniform_bool(Packed, BitOffset + 1), UnpackUniform_bool(Packed, BitOffset + 2));
}

bool4 UnpackUniform_bool4(uint Packed, uint BitOffset)
{
	return bool4(UnpackUniform_bool(Packed, BitOffset), UnpackUniform_bool(Packed, BitOffset + 1), UnpackUniform_bool(Packed, BitOffset + 2), UnpackUniform_bool(Packed, BitOffset + 3));
}

/**
 * Returns the upper 3x3 portion of the LocalToWorld matrix.
 */
MaterialFloat3x3 GetLocalToWorld3x3(FMaterialVertexParameters Parameters)
{
#if DECAL_PRIMITIVE
	return (float3x3)DecalToWorld;
#else
	return DFToFloat3x3(GetPrimitiveData(Parameters).LocalToWorld);
#endif
}

MaterialFloat3x3 GetPreviousLocalToWorld3x3(FMaterialVertexParameters Parameters)
{
#if DECAL_PRIMITIVE
	return (float3x3)DecalToWorld;
#else
	return DFToFloat3x3(GetPrimitiveData(Parameters).PreviousLocalToWorld);
#endif
}

MaterialFloat3x3 GetLocalToWorld3x3(FMaterialPixelParameters Parameters)
{
#if DECAL_PRIMITIVE
	return (float3x3)DecalToWorld;
#else
	return DFToFloat3x3(GetPrimitiveData(Parameters).LocalToWorld);
#endif
}

MaterialFloat3x3 GetLocalToWorld3x3()
{
	return DFToFloat3x3(GetPrimitiveDataFromUniformBuffer().LocalToWorld);
}

FDFInverseMatrix GetWorldToInstanceDF(FMaterialVertexParameters Parameters)
{
	#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS
		return Parameters.InstanceWorldToLocal;
	#else
		return GetPrimitiveData(Parameters).WorldToLocal;
	#endif
}
FWSInverseMatrix GetWorldToInstance(FMaterialVertexParameters Parameters) { return Parameters.LWCData.WorldToInstance; }

FDFInverseMatrix GetWorldToInstanceDF(FMaterialPixelParameters Parameters)
{
	#if HAS_INSTANCE_WORLD_TO_LOCAL_PS
		return Parameters.InstanceWorldToLocal;
	#else
		return GetPrimitiveData(Parameters).WorldToLocal;
	#endif
}
FWSInverseMatrix GetWorldToInstance(FMaterialPixelParameters Parameters) { return Parameters.LWCData.WorldToInstance; }

FDFMatrix GetInstanceToWorldDF(FMaterialVertexParameters Parameters)
{
	#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS
		return Parameters.InstanceLocalToWorld;
	#else
		return GetPrimitiveData(Parameters).LocalToWorld;
	#endif
}
FWSMatrix GetInstanceToWorld(FMaterialVertexParameters Parameters) { return Parameters.LWCData.InstanceToWorld; }

FDFMatrix GetPrevInstanceToWorldDF(FMaterialVertexParameters Parameters)
{
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY || IS_NANITE_PASS
	return Parameters.PrevFrameLocalToWorld;
#else
	return GetPrimitiveData(Parameters).PreviousLocalToWorld;
#endif
}
FWSMatrix GetPrevInstanceToWorld(FMaterialVertexParameters Parameters) { return Parameters.LWCData.PreviousInstanceToWorld; }


FDFMatrix GetInstanceToWorldDF(FMaterialPixelParameters Parameters)
{
	#if HAS_INSTANCE_LOCAL_TO_WORLD_PS
		return Parameters.InstanceLocalToWorld;
	#else
		return GetPrimitiveData(Parameters).LocalToWorld;
	#endif
}
FWSMatrix GetInstanceToWorld(FMaterialPixelParameters Parameters) { return Parameters.LWCData.InstanceToWorld; }

FDFMatrix GetPrevInstanceToWorldDF(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters).PreviousLocalToWorld;
}
FWSMatrix GetPrevInstanceToWorld(FMaterialPixelParameters Parameters) { return Parameters.LWCData.PreviousInstanceToWorld; }

FWSVector3 GetPreViewTranslation(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PreViewTranslation;
}

FWSVector3 GetPreViewTranslation(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PreViewTranslation;
}

FWSVector3 GetPrevPreViewTranslation(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PrevPreViewTranslation;
}

FWSVector3 GetPrevPreViewTranslation(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PrevPreViewTranslation;
}

FWSVector3 GetWorldViewOrigin(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.WorldViewOrigin;
}

FWSVector3 GetWorldViewOrigin(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.WorldViewOrigin;
}

FWSVector3 GetPrevWorldViewOrigin(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PrevWorldViewOrigin;
}

FWSVector3 GetPrevWorldViewOrigin(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PrevWorldViewOrigin;
}

FWSVector3 GetWorldCameraOrigin(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.WorldCameraOrigin;
}

FWSVector3 GetWorldCameraOrigin(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.WorldCameraOrigin;
}

FWSVector3 GetPrevWorldCameraOrigin(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PrevWorldCameraOrigin;
}

FWSVector3 GetPrevWorldCameraOrigin(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PrevWorldCameraOrigin;
}


// World position

float3 GetTranslatedWorldPosition(FMaterialVertexParameters Parameters)
{
	return Parameters.WorldPosition;
}

float3 GetPrevTranslatedWorldPosition(FMaterialVertexParameters Parameters)
{
	// Previous world position and current world position are sharing the
	// same attribute in Parameters, because in BasePassVertexShader.usf
	// and in VelocityShader.usf, we are regenerating a Parameters from
	// VertexFactoryGetPreviousWorldPosition() instead of
	// VertexFactoryGetWorldPosition().
	return GetTranslatedWorldPosition(Parameters);
}

FWSVector3 GetWorldPosition(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.AbsoluteWorldPosition;
}

FWSVector3 GetPrevWorldPosition(FMaterialVertexParameters Parameters)
{
	return WSSubtract(GetPrevTranslatedWorldPosition(Parameters), GetPrevPreViewTranslation(Parameters));
}

FWSVector3 GetWorldPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.AbsoluteWorldPosition;
}

FWSVector3 GetWorldPosition_NoMaterialOffsets(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.WorldPosition_NoOffsets;
}

float3 GetTranslatedWorldPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.WorldPosition_CamRelative;
}

float3 GetTranslatedWorldPosition_NoMaterialOffsets(FMaterialPixelParameters Parameters)
{
	return Parameters.WorldPosition_NoOffsets_CamRelative;
}

FWSVector3 GetParticleWorldPosition(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.ParticleWorldPosition;
}

FWSVector3 GetParticleWorldPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.ParticleWorldPosition;
}

FWSVector3 GetPrevParticleWorldPosition(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PrevParticleWorldPosition;
}

FWSVector3 GetPrevParticleWorldPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PrevParticleWorldPosition;
}

// Periodic World Position

float3 GetPeriodicWorldOrigin(float3 Scale)
{
	// Following three lines can be replaced with `ScaledPos = DFDivide(ResolvedView.PreViewTranslation, Scale)` but this is faster and reasonably precise.
	float3 InvScale = rcp(Scale);
	FDFVector3 ScaledPos = DFTwoProduct(ResolvedView.PreViewTranslation.High, InvScale);
	ScaledPos.Low += ResolvedView.PreViewTranslation.Low * InvScale;
	
	return DFFracDemote(ScaledPos) * Scale;
}

float3 GetPeriodicWorldOrigin_Pow2(float3 Scale)
{
	return DFFmodByPow2Demote(ResolvedView.PreViewTranslation, Scale);
}

// Local position

float3 GetPositionPrimitiveSpace(FMaterialPixelParameters Parameters)
{
	// See also: derivative computation of local position in material translator.
	return DFFastMultiplyDemote(Parameters.AbsoluteWorldPosition, GetPrimitiveData(Parameters).WorldToLocal);
}

float3 GetPositionPrimitiveSpace(FMaterialVertexParameters Parameters)
{
	return Parameters.PositionPrimitiveSpace;
}

float3 GetPrevPositionPrimitiveSpace(FMaterialVertexParameters Parameters)
{
	// Previous position and current position are sharing the
	// same attribute in Parameters, because in BasePassVertexShader.usf
	// and in VelocityShader.usf, we are regenerating a Parameters with
	// GetMaterialVertexParameters(..., bIsPreviousFrame=true)
	return Parameters.PositionPrimitiveSpace;
}

float3 GetPositionPrimitiveSpace_NoMaterialOffsets(FMaterialPixelParameters Parameters)
{
	return DFFastMultiplyDemote(Parameters.WorldPosition_NoOffsets, GetPrimitiveData(Parameters).WorldToLocal);
}

float3 GetPositionInstanceSpace(FMaterialPixelParameters Parameters)
{
	// See also: derivative computation of local position in material translator.
	return DFFastMultiplyDemote(Parameters.AbsoluteWorldPosition, GetWorldToInstanceDF(Parameters));
}

float3 GetPositionInstanceSpace(FMaterialVertexParameters Parameters)
{
	return Parameters.PositionInstanceSpace;
}

float3 GetPrevPositionInstanceSpace(FMaterialVertexParameters Parameters)
{
	// Previous position and current position are sharing the
	// same attribute in Parameters, because in BasePassVertexShader.usf
	// and in VelocityShader.usf, we are regenerating a Parameters with
	// GetMaterialVertexParameters(..., bIsPreviousFrame=true)
	return Parameters.PositionInstanceSpace;
}

float3 GetPositionInstanceSpace_NoMaterialOffsets(FMaterialPixelParameters Parameters)
{
	return DFFastMultiplyDemote(Parameters.WorldPosition_NoOffsets, GetWorldToInstanceDF(Parameters));
}

// Screen position

float4 GetScreenPosition(FMaterialVertexParameters Parameters)
{
	return mul(float4(Parameters.WorldPosition, 1.0f), ResolvedView.TranslatedWorldToClip);
}

float4 GetScreenPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.ScreenPosition;
}

uint GetViewId(FMaterialPixelParameters Parameters)
{
#if MOBILE_MULTI_VIEW
	return Parameters.ViewId;
#else
	return 0;
#endif
}

// Returns the pixel's depth, in world units. Works for both orthographic and perspective projections:
float GetPixelDepth(FMaterialVertexParameters Parameters)
{
	return GetScreenPositionDepth(GetScreenPosition(Parameters));
}

float GetPixelDepth(FMaterialPixelParameters Parameters)
{
	return GetScreenPositionDepth(GetScreenPosition(Parameters));
}

float2 GetSceneTextureUV(FMaterialVertexParameters Parameters)
{
	return ScreenAlignedPosition(GetScreenPosition(Parameters));
}

float2 GetSceneTextureUV(FMaterialPixelParameters Parameters)
{
	return SvPositionToBufferUV(Parameters.SvPosition);
}

float2 GetViewportUV(FMaterialVertexParameters Parameters)
{
#if POST_PROCESS_MATERIAL
	return Parameters.WorldPosition.xy;
#else
	return BufferUVToViewportUV(GetSceneTextureUV(Parameters));
#endif
}

float2 GetPixelPosition(FMaterialVertexParameters Parameters)
{
	return GetViewportUV(Parameters) * View.ViewSizeAndInvSize.xy;
}

#if POST_PROCESS_MATERIAL

float2 GetPixelPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.SvPosition.xy - float2(PostProcessOutput_ViewportMin);
}

float2 GetViewportUV(FMaterialPixelParameters Parameters)
{
	return GetPixelPosition(Parameters) * PostProcessOutput_ViewportSizeInverse;
}

#else

float2 GetPixelPosition(FMaterialPixelParameters Parameters)
{
	return Parameters.SvPosition.xy - ResolvedView.ViewRectMin.xy;
}

float2 GetViewportUV(FMaterialPixelParameters Parameters)
{
	return SvPositionToViewportUV(Parameters.SvPosition);
}

#endif

float GetWaterWaveParamIndex(FMaterialPixelParameters Parameters)
{
#if WATER_MESH_FACTORY
	return (float)Parameters.WaterWaveParamIndex;
#else
	return 0.0f;
#endif
}

float GetWaterWaveParamIndex(FMaterialVertexParameters Parameters)
{
#if WATER_MESH_FACTORY
	return (float)Parameters.WaterWaveParamIndex;
#else
	return 0.0f;
#endif
}

float PostVolumeUserFlagTest(float CompareValue)
{
	return (((int)1 << (int)CompareValue) & View.PostVolumeUserFlags) ? 1.0 : 0.0;
}

// Returns whether a scene texture id is a for a post process input or not.
bool IsPostProcessInputSceneTexture(const uint SceneTextureId)
{
	return (SceneTextureId >= PPI_PostProcessInput0 && SceneTextureId <= PPI_PostProcessInput6);
}

// Returns the view size and texel size in a given scene texture.
float4 GetSceneTextureViewSize(const uint SceneTextureId)
{
	#if POST_PROCESS_MATERIAL
	if (IsPostProcessInputSceneTexture(SceneTextureId))
	{
		switch (SceneTextureId)
		{
		case PPI_PostProcessInput0:
			return float4(PostProcessInput_0_ViewportSize, PostProcessInput_0_ViewportSizeInverse);
		case PPI_PostProcessInput1:
			return float4(PostProcessInput_1_ViewportSize, PostProcessInput_1_ViewportSizeInverse);
		case PPI_PostProcessInput2:
			return float4(PostProcessInput_2_ViewportSize, PostProcessInput_2_ViewportSizeInverse);
		case PPI_PostProcessInput3:
			return float4(PostProcessInput_3_ViewportSize, PostProcessInput_3_ViewportSizeInverse);
		case PPI_PostProcessInput4:
			return float4(PostProcessInput_4_ViewportSize, PostProcessInput_4_ViewportSizeInverse);
		default:
			return float4(0, 0, 0, 0);
		}
	}
	#endif
	return ResolvedView.ViewSizeAndInvSize;
}

// Returns the buffer size and texel size (inverse buffer size) for a given scene texture.
float4 GetSceneTextureBufferSize(const uint SceneTextureId)
{
#if POST_PROCESS_MATERIAL
	if (IsPostProcessInputSceneTexture(SceneTextureId))
	{
		switch (SceneTextureId)
		{
		case PPI_PostProcessInput0:
			return float4(PostProcessInput_0_Extent, PostProcessInput_0_ExtentInverse);
		case PPI_PostProcessInput1:
			return float4(PostProcessInput_1_Extent, PostProcessInput_1_ExtentInverse);
		case PPI_PostProcessInput2:
			return float4(PostProcessInput_2_Extent, PostProcessInput_2_ExtentInverse);
		case PPI_PostProcessInput3:
			return float4(PostProcessInput_3_Extent, PostProcessInput_3_ExtentInverse);
		case PPI_PostProcessInput4:
			return float4(PostProcessInput_4_Extent, PostProcessInput_4_ExtentInverse);
		default:
			return float4(0, 0, 0, 0);
		}
	}
#endif
	return ResolvedView.BufferSizeAndInvSize;
}

// Return the buffer UV min and max for a given scene texture id.
float4 GetSceneTextureUVMinMax(const uint SceneTextureId)
{
	#if POST_PROCESS_MATERIAL
	if (IsPostProcessInputSceneTexture(SceneTextureId))
	{
		switch (SceneTextureId)
	{
		case PPI_PostProcessInput0:
			return float4(PostProcessInput_0_UVViewportBilinearMin, PostProcessInput_0_UVViewportBilinearMax);
		case PPI_PostProcessInput1:
			return float4(PostProcessInput_1_UVViewportBilinearMin, PostProcessInput_1_UVViewportBilinearMax);
		case PPI_PostProcessInput2:
			return float4(PostProcessInput_2_UVViewportBilinearMin, PostProcessInput_2_UVViewportBilinearMax);
		case PPI_PostProcessInput3:
			return float4(PostProcessInput_3_UVViewportBilinearMin, PostProcessInput_3_UVViewportBilinearMax);
		case PPI_PostProcessInput4:
			return float4(PostProcessInput_4_UVViewportBilinearMin, PostProcessInput_4_UVViewportBilinearMax);
		default:
			return float4(0, 0, 1, 1);
		}
	}
	#endif

	return View.BufferBilinearUVMinMax;
}

// Transforms viewport UV to scene texture's UV.
MaterialFloat2 ViewportUVToSceneTextureUV(MaterialFloat2 ViewportUV, const uint SceneTextureId)
{
	#if POST_PROCESS_MATERIAL
	if (IsPostProcessInputSceneTexture(SceneTextureId))
	{
		switch (SceneTextureId)
		{
		case PPI_PostProcessInput0:
			return ViewportUV * PostProcessInput_0_UVViewportSize + PostProcessInput_0_UVViewportMin;
		case PPI_PostProcessInput1:
			return ViewportUV * PostProcessInput_1_UVViewportSize + PostProcessInput_1_UVViewportMin;
		case PPI_PostProcessInput2:
			return ViewportUV * PostProcessInput_2_UVViewportSize + PostProcessInput_2_UVViewportMin;
		case PPI_PostProcessInput3:
			return ViewportUV * PostProcessInput_3_UVViewportSize + PostProcessInput_3_UVViewportMin;
		case PPI_PostProcessInput4:
			return ViewportUV * PostProcessInput_4_UVViewportSize + PostProcessInput_4_UVViewportMin;
		default:
			return ViewportUV;
		}
	}
	#endif

	return ViewportUVToBufferUV(ViewportUV);
}

// Manually clamp scene texture UV as if using a clamp sampler.
MaterialFloat2 ClampSceneTextureUV(MaterialFloat2 BufferUV, const uint SceneTextureId)
{
	float4 MinMax = GetSceneTextureUVMinMax(SceneTextureId);

	return clamp(BufferUV, MinMax.xy, MinMax.zw);
}

// Get default scene texture's UV.
MaterialFloat2 GetDefaultSceneTextureUV(FMaterialVertexParameters Parameters, const uint SceneTextureId)
{
	return GetSceneTextureUV(Parameters);
}

// Get default scene texture's UV.
MaterialFloat2 GetDefaultSceneTextureUV(FMaterialPixelParameters Parameters, const uint SceneTextureId)
{
	#if POST_PROCESS_MATERIAL
		return ViewportUVToSceneTextureUV(GetViewportUV(Parameters), SceneTextureId);
	#else
		return GetSceneTextureUV(Parameters);
	#endif
}


#if DECAL_PRIMITIVE && NUM_MATERIAL_TEXCOORDS
	/*
	 * Material node DecalMipmapLevel's code designed to avoid the 2x2 pixels artefacts on the edges around where the decal
	 * is projected to. The technique is fetched from (http://www.humus.name/index.php?page=3D&ID=84).
	 *
	 * The problem around edges of the meshes, is that the hardware computes the mipmap level according to ddx(uv) and ddy(uv),
	 * but since the pixel shader are invocated by group of 2x2 pixels, then on edges some pixel might be getting the
	 * current depth of an differet mesh that the other pixel of the same groups. If this mesh is very far from the other
	 * mesh of the same group of pixel, then one of the delta might be very big, leading to choosing a low mipmap level for this
	 * group of 4 pixels, causing the artefacts.
	 */
	float2 ComputeDecalUVFromSvPosition(float4 SvPosition)
	{
		half DeviceZ = LookupDeviceZ(SvPositionToBufferUV(SvPosition));

		SvPosition.z = DeviceZ;

		float4 DecalVector = mul(float4(SvPosition.xyz,1), SvPositionToDecal);
		DecalVector.xyz /= DecalVector.w;
		DecalVector = DecalVector * 0.5f + 0.5f;
		DecalVector.xyz = DecalVector.zyx;
		return DecalVector.xy;
	}

	float2 ComputeDecalDDX(FMaterialPixelParameters Parameters)
	{
		/*
		 * Assuming where in a pixel shader invocation, then we compute manualy compute two d(uv)/d(x)
		 * with the pixels's left and right neighbours.
		 */
		float4 ScreenDeltaX = float4(1, 0, 0, 0);
		float2 UvDiffX0 = Parameters.TexCoords[0] - ComputeDecalUVFromSvPosition(Parameters.SvPosition - ScreenDeltaX);
		float2 UvDiffX1 = ComputeDecalUVFromSvPosition(Parameters.SvPosition + ScreenDeltaX) - Parameters.TexCoords[0];

		/*
		 * So we have two diff on the X axis, we want the one that has the smallest length
		 * to avoid the 2x2 pixels mipmap artefacts on the edges. 
		 */
		return dot(UvDiffX0, UvDiffX0) < dot(UvDiffX1, UvDiffX1) ? UvDiffX0 : UvDiffX1;
	}

	float2 ComputeDecalDDY(FMaterialPixelParameters Parameters)
	{
		// do same for the Y axis
		float4 ScreenDeltaY = float4(0, 1, 0, 0);
		float2 UvDiffY0 = Parameters.TexCoords[0] - ComputeDecalUVFromSvPosition(Parameters.SvPosition - ScreenDeltaY);
		float2 UvDiffY1 = ComputeDecalUVFromSvPosition(Parameters.SvPosition + ScreenDeltaY) - Parameters.TexCoords[0];

		return dot(UvDiffY0, UvDiffY0) < dot(UvDiffY1, UvDiffY1) ? UvDiffY0 : UvDiffY1;
	}

	float ComputeDecalMipmapLevel(FMaterialPixelParameters Parameters, float2 TextureSize)
	{
		float2 UvPixelDiffX = ComputeDecalDDX(Parameters) * TextureSize;
		float2 UvPixelDiffY = ComputeDecalDDY(Parameters) * TextureSize;

		// Computes the mipmap level
		float MaxDiff = max(dot(UvPixelDiffX, UvPixelDiffX), dot(UvPixelDiffY, UvPixelDiffY));
		return 0.5 * log2(MaxDiff);
	}
#else // DECAL_PRIMITIVE && NUM_MATERIAL_TEXCOORDS
	float2 ComputeDecalDDX(FMaterialPixelParameters Parameters)
	{
		return 0.0f;
	}
	
	float2 ComputeDecalDDY(FMaterialPixelParameters Parameters)
	{
		return 0.0f;
	}

	float ComputeDecalMipmapLevel(FMaterialPixelParameters Parameters, float2 TextureSize)
	{
		return 0.0f;
	}
#endif // DECAL_PRIMITIVE && NUM_MATERIAL_TEXCOORDS

	
	FWSVector3 GetActorWorldPosition(FMaterialVertexParameters Parameters) { return Parameters.LWCData.ActorWorldPosition; }

	FWSVector3 GetActorWorldPosition(FMaterialPixelParameters Parameters) { return Parameters.LWCData.ActorWorldPosition; }

	FWSVector3 GetPreviousActorWorldPosition(FMaterialVertexParameters Parameters)
	{
		float3 InstanceSpacePos = WSMultiplyDemote(GetActorWorldPosition(Parameters), GetWorldToInstance(Parameters));
		return WSMultiply(InstanceSpacePos, GetPrevInstanceToWorld(Parameters));
	}

	FWSVector3 GetPreviousActorWorldPosition(FMaterialPixelParameters Parameters)
	{
		float3 InstanceSpacePos = WSMultiplyDemote(GetActorWorldPosition(Parameters), GetWorldToInstance(Parameters));
		return WSMultiply(InstanceSpacePos, GetPrevInstanceToWorld(Parameters));
	}

	float3 GetActorTranslatedWorldPosition(FMaterialVertexParameters Parameters)
	{
#if PRIMITIVE_HAS_TILEOFFSET_DATA && VIEW_HAS_TILEOFFSET_DATA
		return LWCToFloat(LWCAdd(GetPrimitiveData(Parameters).ActorWorldPositionTO, ResolvedView.TileOffset.PreViewTranslation));
#else
		return DFFastToTranslatedWorld(GetPrimitiveData(Parameters).ActorWorldPosition, ResolvedView.PreViewTranslation);
#endif
	}

	float3 GetActorTranslatedWorldPosition(FMaterialPixelParameters Parameters)
	{
#if PRIMITIVE_HAS_TILEOFFSET_DATA && VIEW_HAS_TILEOFFSET_DATA
		return LWCToFloat(LWCAdd(GetPrimitiveData(Parameters).ActorWorldPositionTO, ResolvedView.TileOffset.PreViewTranslation));
#else
		return DFFastToTranslatedWorld(GetPrimitiveData(Parameters).ActorWorldPosition, ResolvedView.PreViewTranslation);
#endif
	}

	float3 GetPreviousActorTranslatedWorldPosition(FMaterialVertexParameters Parameters)
	{
		float3 InstanceToActor = WSMultiplyDemote(GetActorWorldPosition(Parameters), GetWorldToInstance(Parameters));
		return WSSubtractDemote(WSMultiply(InstanceToActor, Parameters.LWCData.PreviousLocalToWorld), WSNegate(GetPreViewTranslation(Parameters)));
	}

	float3 GetPreviousActorTranslatedWorldPosition(FMaterialPixelParameters Parameters)
	{
		float3 InstanceToActor = WSMultiplyDemote(GetActorWorldPosition(Parameters), GetWorldToInstance(Parameters));
		return WSSubtractDemote(WSMultiply(InstanceToActor, Parameters.LWCData.PreviousLocalToWorld), WSNegate(GetPreViewTranslation(Parameters)));
	}
	
	float3 GetObjectOrientation(FMaterialVertexParameters Parameters)
	{
	#if DECAL_PRIMITIVE
		return DecalOrientation.xyz;
	#else
		return GetPrimitiveData(Parameters).ObjectOrientation;
	#endif
	}

	float3 GetObjectOrientation(FMaterialPixelParameters Parameters)
	{
	#if DECAL_PRIMITIVE
		return DecalOrientation.xyz;
	#else
		return GetPrimitiveData(Parameters).ObjectOrientation;
	#endif
	}

	float4 DecalColor()
	{
	#if DECAL_PRIMITIVE
		return DecalColorParam;
	#else
		return float4(1.0f, 1.0f, 1.0f, 1.0f);
	#endif
	}

	float DecalLifetimeOpacity()
	{
	#if DECAL_PRIMITIVE
		return DecalParams.y;
	#else
		return 0.0f;
	#endif
	}

/** Per Instance Custom Data Getter (Pixel Shader Variant - Visibility Buffer) */
float GetPerInstanceCustomData(FMaterialPixelParameters Parameters, int Index, float DefaultValue)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	#if SHADER_USES_PRIMITIVE_UBO
		// Note: Not actually primitive ID here (DrawInstanceId)
		return LoadInstanceCustomDataElementUBO(Parameters.PrimitiveId, FloatIndex, DefaultValue);
	#else
		BRANCH
		if (FloatIndex < Parameters.CustomDataCount)
		{
			const uint   Float4Offset = Parameters.CustomDataOffset + (FloatIndex >> 2u);
			const float4 Float4Packed = LoadInstancePayloadDataElement(Float4Offset);
			return Float4Packed[FloatIndex & 0x3u];
		}
	#endif
#elif USE_INSTANCING && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	BRANCH
	if (FloatIndex < InstanceVF.NumCustomDataFloats)
	{
		const uint InstanceDataIndex = asuint(Parameters.PerInstanceParams.w);
		const uint BufferStartIndex = InstanceDataIndex * InstanceVF.NumCustomDataFloats;
		return InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex];
	}
#endif
	return DefaultValue;
}

// Per Instance Custom Data Getter (Vertex Shader Only)
/** Get the per-instance custom data when instancing */
float GetPerInstanceCustomData(FMaterialVertexParameters Parameters, int Index, float DefaultValue)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	#if SHADER_USES_PRIMITIVE_UBO
		// Note: Not actually primitive ID here (DrawInstanceId)
		return LoadInstanceCustomDataElementUBO(Parameters.PrimitiveId, FloatIndex, DefaultValue);
	#else
		BRANCH
		if (FloatIndex < Parameters.CustomDataCount)
		{
			const uint   Float4Offset = Parameters.CustomDataOffset + (FloatIndex >> 2u);
			const float4 Float4Packed = LoadInstancePayloadDataElement(Float4Offset);
			return Float4Packed[FloatIndex & 0x3u];
		}
	#endif
#elif USE_INSTANCING && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	BRANCH
	if (FloatIndex < InstanceVF.NumCustomDataFloats)
	{
		const uint InstanceDataIndex = Parameters.InstanceId + Parameters.InstanceOffset;
		const uint BufferStartIndex = InstanceDataIndex * InstanceVF.NumCustomDataFloats;
		return InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex];
	}
#endif

	return DefaultValue;
}

/** Per Instance Custom Data Getter (Pixel Shader Variant - Visibility Buffer) */
/** Get the per-instance custom data when instancing */
MaterialFloat3 GetPerInstanceCustomData3Vector(FMaterialPixelParameters Parameters, int Index, MaterialFloat3 DefaultValue)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && USES_PER_INSTANCE_CUSTOM_DATA
	return float3(GetPerInstanceCustomData(Parameters, Index + 0, DefaultValue.x),
		GetPerInstanceCustomData(Parameters, Index + 1, DefaultValue.y),
		GetPerInstanceCustomData(Parameters, Index + 2, DefaultValue.z));

#elif USE_INSTANCING && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	BRANCH
	if (FloatIndex + 2 < InstanceVF.NumCustomDataFloats)
	{
		const uint InstanceDataIndex = asuint(Parameters.PerInstanceParams.w);
		const uint BufferStartIndex = InstanceDataIndex * InstanceVF.NumCustomDataFloats;
		return float3(InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex],
			InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex + 1],
			InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex + 2]);
	}
#endif
	return DefaultValue;
}

// Per Instance Custom Data Getter (Vertex Shader Only)
/** Get the per-instance custom data when instancing */
MaterialFloat3 GetPerInstanceCustomData3Vector(FMaterialVertexParameters Parameters, int Index, MaterialFloat3 DefaultValue)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && USES_PER_INSTANCE_CUSTOM_DATA
		return float3(GetPerInstanceCustomData(Parameters, Index, DefaultValue.x), 
					  GetPerInstanceCustomData(Parameters, Index + 1, DefaultValue.y),
					  GetPerInstanceCustomData(Parameters, Index + 2, DefaultValue.z));
#elif USE_INSTANCING && USES_PER_INSTANCE_CUSTOM_DATA
	const uint FloatIndex = uint(Index);
	BRANCH
	if (FloatIndex + 2 < InstanceVF.NumCustomDataFloats)
	{
		const uint InstanceDataIndex = Parameters.InstanceId + Parameters.InstanceOffset;

		const uint BufferStartIndex = InstanceDataIndex * InstanceVF.NumCustomDataFloats;
		return float3(InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex], 
		              InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex + 1],
					  InstanceVF.InstanceCustomDataBuffer[BufferStartIndex + FloatIndex + 2]);
	}
#endif

	return DefaultValue;
}

/** Transforms a vector from tangent space to view space */
MaterialFloat3 TransformTangentVectorToView(FMaterialPixelParameters Parameters, MaterialFloat3 InTangentVector)
{
	// Transform from tangent to world, and then to view space
	return mul(mul(InTangentVector, Parameters.TangentToWorld), (MaterialFloat3x3)ResolvedView.TranslatedWorldToView);
}

/** Local to world */
FDFMatrix GetLocalToWorldDF(FMaterialVertexParameters Parameters)
{
#if DECAL_PRIMITIVE
	return MakeDFMatrix(DecalPositionHigh.xyz, DecalToWorld);
#elif USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY
	return Parameters.InstanceLocalToWorld;
#else
	return GetPrimitiveData(Parameters).LocalToWorld;
#endif
}

FWSMatrix GetLocalToWorld(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.LocalToWorld;
}

FDFMatrix GetLocalToWorldDF(FMaterialPixelParameters Parameters)
{
#if DECAL_PRIMITIVE
	return MakeDFMatrix(DecalPositionHigh.xyz, DecalToWorld);
#else
	return GetPrimitiveData(Parameters).LocalToWorld;
#endif
}

FWSMatrix GetLocalToWorld(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.LocalToWorld;
}

FDFMatrix GetPrevLocalToWorldDF(FMaterialVertexParameters Parameters)
{
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY
	return Parameters.PrevFrameLocalToWorld;
#else
	return GetPrimitiveData(Parameters).PreviousLocalToWorld;
#endif
}

FWSMatrix GetPrevLocalToWorld(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PreviousLocalToWorld;
}

FDFMatrix GetPrevLocalToWorldDF(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters).PreviousLocalToWorld;
}

FWSMatrix GetPrevLocalToWorld(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PreviousLocalToWorld;
}

/** World to local*/
FDFInverseMatrix GetWorldToLocalDF(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters).WorldToLocal;
}

FWSInverseMatrix GetWorldToLocal(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.WorldToLocal;
}

FDFInverseMatrix GetWorldToLocalDF(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters).WorldToLocal;
}

FWSInverseMatrix GetWorldToLocal(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.WorldToLocal;
}

FDFInverseMatrix GetPrevWorldToLocalDF(FMaterialVertexParameters Parameters)
{
	return GetPrimitiveData(Parameters).PreviousWorldToLocal;
}

FWSInverseMatrix GetPrevWorldToLocal(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.PreviousWorldToLocal;
}

FDFInverseMatrix GetPrevWorldToLocalDF(FMaterialPixelParameters Parameters)
{
	return GetPrimitiveData(Parameters).PreviousWorldToLocal;
}

FWSInverseMatrix GetPrevWorldToLocal(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.PreviousWorldToLocal;
}

/** Transforms a vector from local space to world space (PS version) */
MaterialFloat3 TransformLocalVectorToWorld(FMaterialPixelParameters Parameters, MaterialFloat3 InLocalVector)
{
	return mul(InLocalVector, GetLocalToWorld3x3(Parameters));
}

/** Transforms a vector from local space to world space (VS version) */
MaterialFloat3 TransformLocalVectorToWorld(FMaterialVertexParameters Parameters,MaterialFloat3 InLocalVector)
{
	#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY
		return DFMultiplyVector(InLocalVector, Parameters.InstanceLocalToWorld);
	#else
		return mul(InLocalVector, GetLocalToWorld3x3(Parameters));
	#endif
}

/** Transforms a vector from local space to previous frame world space (VS version) */
MaterialFloat3 TransformLocalVectorToPrevWorld(FMaterialVertexParameters Parameters, MaterialFloat3 InLocalVector)
{
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY
	return DFMultiplyVector(InLocalVector, Parameters.PrevFrameLocalToWorld);
#else
	return mul(InLocalVector, GetPreviousLocalToWorld3x3(Parameters));
#endif
}

/** Transforms a position from local space to absolute world space */
FWSVector3 TransformLocalPositionToWorld(FMaterialPixelParameters Parameters,float3 InLocalPosition)
{
	return WSMultiply(InLocalPosition, GetLocalToWorld(Parameters));
}

/** Transforms a position from local space to absolute world space */
FWSVector3 TransformLocalPositionToWorld(FMaterialVertexParameters Parameters,float3 InLocalPosition)
{
	return WSMultiply(InLocalPosition, GetLocalToWorld(Parameters));
}

/** Transforms a position from local space to previous frame absolute world space */
FWSVector3 TransformLocalPositionToPrevWorld(FMaterialVertexParameters Parameters,float3 InLocalPosition)
{
	return WSMultiply(InLocalPosition, GetPrevLocalToWorld(Parameters));
}

#if MATERIAL_DOMAIN_LIGHTFUNCTION && !LIGHT_ATLAS
#ifndef CameraRelativeLightPosition
	float3 CameraRelativeLightPosition;
#endif
#endif

/** Return the object's position in world space */
FWSVector3 GetObjectWorldPosition(FMaterialPixelParameters Parameters)
{
#if HAS_PRIMITIVE_UNIFORM_BUFFER || DECAL_PRIMITIVE
	return Parameters.LWCData.ObjectWorldPosition;
#elif MATERIAL_DOMAIN_LIGHTFUNCTION && !LIGHT_ATLAS
	/**
	* Light functions return light position in the Pixel shader only.
	* Light Atlas cannot return positional values, so default to float3(0.0)
	*/
	return DFToWS(DFFastSubtract(CameraRelativeLightPosition, PrimaryView.PreViewTranslation));
#else
	return WSPromote(float3(0.0f, 0.0f, 0.0f));
#endif //HAS_PRIMITIVE_UNIFORM_BUFFER
}

/** Return the object's position in world space. For instanced meshes, this returns the instance position. */
FWSVector3 GetObjectWorldPosition(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.ObjectWorldPosition;
}
float3 GetObjectTranslatedWorldPosition(FMaterialPixelParameters Parameters)
{
#if HAS_PRIMITIVE_UNIFORM_BUFFER || DECAL_PRIMITIVE

#if PRIMITIVE_HAS_TILEOFFSET_DATA && VIEW_HAS_TILEOFFSET_DATA
	return LWCToFloat(LWCAdd(GetPrimitiveData(Parameters).ObjectWorldPositionTO, ResolvedView.TileOffset.PreViewTranslation));
#else
	return DFFastToTranslatedWorld(GetPrimitiveData(Parameters).ObjectWorldPosition, ResolvedView.PreViewTranslation);
#endif //PRIMITIVE_HAS_TILEOFFSET_DATA && VIEW_HAS_TILEOFFSET_DATA

#elif MATERIAL_DOMAIN_LIGHTFUNCTION && !LIGHT_ATLAS
	/**
	* Light functions return light position in the Pixel shader only.
	* Light Atlas cannot return positional values, so default to float3(0.0)
	*/
	return CameraRelativeLightPosition;
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif //HAS_PRIMITIVE_UNIFORM_BUFFER
}

float3 GetObjectTranslatedWorldPosition(FMaterialVertexParameters Parameters)
{
#if PRIMITIVE_HAS_TILEOFFSET_DATA && VIEW_HAS_TILEOFFSET_DATA
	return LWCToFloat(LWCAdd(GetPrimitiveData(Parameters).ObjectWorldPositionTO, ResolvedView.TileOffset.PreViewTranslation));
#else
	return DFFastToTranslatedWorld(GetPrimitiveData(Parameters).ObjectWorldPosition, ResolvedView.PreViewTranslation);
#endif
}

FWSInverseMatrix GetWorldToParticle(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.WorldToParticle;
}

FWSInverseMatrix GetWorldToParticle(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.WorldToParticle;
}

FWSMatrix GetParticleToWorld(FMaterialPixelParameters Parameters)
{
	return Parameters.LWCData.ParticleToWorld;
}

FWSMatrix GetParticleToWorld(FMaterialVertexParameters Parameters)
{
	return Parameters.LWCData.ParticleToWorld;
}

FMaterialLWCData MakeMaterialLWCData(FMaterialVertexParameters Parameters)
{
	FMaterialLWCData Result = (FMaterialLWCData)0;

#if VIEW_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
	Result.PreViewTranslation = ResolvedView.TileOffset.PreViewTranslation;
	Result.PrevPreViewTranslation = ResolvedView.TileOffset.PrevPreViewTranslation;
	Result.WorldViewOrigin = ResolvedView.TileOffset.WorldViewOrigin;
	Result.PrevWorldViewOrigin = ResolvedView.TileOffset.PrevWorldViewOrigin;
	Result.WorldCameraOrigin = ResolvedView.TileOffset.WorldCameraOrigin;
	Result.PrevWorldCameraOrigin = ResolvedView.TileOffset.PrevWorldCameraOrigin;
#else
	Result.PreViewTranslation = DFToWS(ResolvedView.PreViewTranslation);
	Result.PrevPreViewTranslation = DFToWS(ResolvedView.PrevPreViewTranslation);
	Result.WorldViewOrigin = DFToWS(ResolvedView.WorldViewOrigin);
	Result.PrevWorldViewOrigin = DFToWS(ResolvedView.PrevWorldViewOrigin);
	Result.WorldCameraOrigin = DFToWS(ResolvedView.WorldCameraOrigin);
	Result.PrevWorldCameraOrigin = DFToWS(ResolvedView.PrevWorldCameraOrigin);
#endif

	Result.AbsoluteWorldPosition = WSSubtract(Parameters.WorldPosition, Result.PreViewTranslation);

	Result.InstanceToWorld = DFToWS(GetInstanceToWorldDF(Parameters));
	Result.WorldToInstance = DFToWS(GetWorldToInstanceDF(Parameters));
	Result.PreviousInstanceToWorld = DFToWS(GetPrevInstanceToWorldDF(Parameters));

	Result.LocalToWorld = DFToWS(GetLocalToWorldDF(Parameters));
	Result.WorldToLocal = DFToWS(GetWorldToLocalDF(Parameters));
	Result.PreviousLocalToWorld = DFToWS(GetPrevLocalToWorldDF(Parameters));
	Result.PreviousWorldToLocal = DFToWS(GetPrevWorldToLocalDF(Parameters));

#if DECAL_PRIMITIVE
	/*
	 * Deferred decal don't have a Primitive uniform buffer, because we don't know on which primitive the decal
	 * is being projected to. But the user may still need to get the decal's actor world position.
	 * So instead of setting up a primitive buffer that may cost to much CPU effort to be almost never used,
	 * we directly fetch this value from the DeferredDecal.usf specific uniform variable DecalToWorld.
	 */
	Result.ObjectWorldPosition = DFToWS(MakeDFVector3(DecalPositionHigh.xyz, DecalToWorld[3].xyz));
	Result.ActorWorldPosition = Result.ObjectWorldPosition;

#else //DECAL_PRIMITIVE
	
#if USE_INSTANCING || USE_INSTANCE_CULLING || IS_MESHPARTICLE_FACTORY
	// TODO: Rename PRIMITIVE_SCENE_DATA_FLAG_UNUSED_2
	bool bEnableObjectWorldPositionFix = (GetPrimitiveData(Parameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_UNUSED_2) != 0;
	if (bEnableObjectWorldPositionFix)
	{
#if PRIMITIVE_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
		Result.ObjectWorldPosition = GetPrimitiveData(Parameters).ObjectWorldPositionTO;
#else
		Result.ObjectWorldPosition = DFToWS(GetPrimitiveData(Parameters).ObjectWorldPosition);
#endif
	}
	else
	{
#if MATERIAL_LWC_ENABLED
		Result.ObjectWorldPosition = WSGetOrigin(DFToWS(Parameters.InstanceLocalToWorld));
#else //MATERIAL_LWC_ENABLED
		Result.ObjectWorldPosition = WSPromote(Parameters.InstanceLocalToWorld.PostTranslation + Parameters.InstanceLocalToWorld.M[3].xyz);
#endif //MATERIAL_LWC_ENABLED
	}
#elif PRIMITIVE_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
	Result.ObjectWorldPosition = GetPrimitiveData(Parameters).ObjectWorldPositionTO;
#else
	Result.ObjectWorldPosition = DFToWS(GetPrimitiveData(Parameters).ObjectWorldPosition);
#endif

#if PRIMITIVE_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
	Result.ActorWorldPosition = GetPrimitiveData(Parameters).ActorWorldPositionTO;
#else
	Result.ActorWorldPosition = DFToWS(GetPrimitiveData(Parameters).ActorWorldPosition);
#endif

#endif //DECAL_PRIMITIVE

	Result.ParticleToWorld = DFFastToWS(Parameters.Particle.ParticleToWorld);
	Result.WorldToParticle = DFFastToWS(Parameters.Particle.WorldToParticle);
	Result.ParticleWorldPosition = WSSubtract(Parameters.Particle.TranslatedWorldPositionAndSize.xyz, Result.PreViewTranslation);
	Result.PrevParticleWorldPosition = WSSubtract(Parameters.Particle.PrevTranslatedWorldPositionAndSize.xyz, Result.PrevPreViewTranslation);
	
	return Result;
}

FMaterialLWCData MakeMaterialLWCData(FMaterialPixelParameters Parameters)
{
	FMaterialLWCData Result = (FMaterialLWCData)0;

#if VIEW_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
	Result.PreViewTranslation = ResolvedView.TileOffset.PreViewTranslation;
	Result.PrevPreViewTranslation = ResolvedView.TileOffset.PrevPreViewTranslation;
	Result.WorldViewOrigin = ResolvedView.TileOffset.WorldViewOrigin;
	Result.PrevWorldViewOrigin = ResolvedView.TileOffset.PrevWorldViewOrigin;
	Result.WorldCameraOrigin = ResolvedView.TileOffset.WorldCameraOrigin;
	Result.PrevWorldCameraOrigin = ResolvedView.TileOffset.PrevWorldCameraOrigin;
#else
	Result.PreViewTranslation = DFToWS(ResolvedView.PreViewTranslation);
	Result.PrevPreViewTranslation = DFToWS(ResolvedView.PrevPreViewTranslation);
	Result.WorldViewOrigin = DFToWS(ResolvedView.WorldViewOrigin);
	Result.PrevWorldViewOrigin = DFToWS(ResolvedView.PrevWorldViewOrigin);
	Result.WorldCameraOrigin = DFToWS(ResolvedView.WorldCameraOrigin);
	Result.PrevWorldCameraOrigin = DFToWS(ResolvedView.PrevWorldCameraOrigin);
#endif

	Result.AbsoluteWorldPosition = WSSubtract(Parameters.WorldPosition_CamRelative, Result.PreViewTranslation);
	Result.WorldPosition_NoOffsets = WSSubtract(Parameters.WorldPosition_NoOffsets_CamRelative, Result.PreViewTranslation);

	Result.InstanceToWorld = DFToWS(GetInstanceToWorldDF(Parameters));
	Result.WorldToInstance = DFFastToWS(GetWorldToInstanceDF(Parameters));
	Result.PreviousInstanceToWorld = DFToWS(GetPrevInstanceToWorldDF(Parameters));

	Result.LocalToWorld = DFToWS(GetLocalToWorldDF(Parameters));
	Result.WorldToLocal = DFFastToWS(GetWorldToLocalDF(Parameters));
	Result.PreviousLocalToWorld = DFToWS(GetPrevLocalToWorldDF(Parameters));
	Result.PreviousWorldToLocal = DFFastToWS(GetPrevWorldToLocalDF(Parameters));

#if DECAL_PRIMITIVE
	/*
	 * Deferred decal don't have a Primitive uniform buffer, because we don't know on which primitive the decal
	 * is being projected to. But the user may still need to get the decal's actor world position.
	 * So instead of setting up a primitive buffer that may cost to much CPU effort to be almost never used,
	 * we directly fetch this value from the DeferredDecal.usf specific uniform variable DecalToWorld.
	 */
	Result.ObjectWorldPosition = DFToWS(MakeDFVector3(DecalPositionHigh.xyz, DecalToWorld[3].xyz));
	Result.ActorWorldPosition = Result.ObjectWorldPosition;

#else //DECAL_PRIMITIVE

#if PRIMITIVE_HAS_TILEOFFSET_DATA && WSVECTOR_IS_TILEOFFSET
	Result.ObjectWorldPosition = GetPrimitiveData(Parameters).ObjectWorldPositionTO;
	Result.ActorWorldPosition = GetPrimitiveData(Parameters).ActorWorldPositionTO;
#else
	Result.ObjectWorldPosition = DFToWS(GetPrimitiveData(Parameters).ObjectWorldPosition);
	Result.ActorWorldPosition = DFToWS(GetPrimitiveData(Parameters).ActorWorldPosition);
#endif

#endif //DECAL_PRIMITIVE

	Result.ParticleToWorld = DFFastToWS(Parameters.Particle.ParticleToWorld);
	Result.WorldToParticle = DFFastToWS(Parameters.Particle.WorldToParticle);
	Result.ParticleWorldPosition = WSSubtract(Parameters.Particle.TranslatedWorldPositionAndSize.xyz, Result.PreViewTranslation);
	Result.PrevParticleWorldPosition = WSSubtract(Parameters.Particle.PrevTranslatedWorldPositionAndSize.xyz, Result.PrevPreViewTranslation);

	return Result;
}

/** Get the per-instance random value when instancing */
float GetPerInstanceRandom(FMaterialVertexParameters Parameters)
{
	return Parameters.PerInstanceRandom;
}

/** Get the per-instance random value when instancing */
float GetPerInstanceRandom(FMaterialPixelParameters Parameters)
{
	return Parameters.PerInstanceRandom;
}

/** Get the per-instance fade-out amount when instancing */
float GetPerInstanceFadeAmount(FMaterialPixelParameters Parameters)
{
#if USES_PER_INSTANCE_FADE_AMOUNT && (USE_INSTANCING || USE_INSTANCE_CULLING)
	return float(Parameters.PerInstanceParams.x);
#else
	return float(1.0);
#endif
}

/** Get the per-instance fade-out amount when instancing */
float GetPerInstanceFadeAmount(FMaterialVertexParameters Parameters)
{
#if USES_PER_INSTANCE_FADE_AMOUNT && (USE_INSTANCING || USE_INSTANCE_CULLING)
	return float(Parameters.PerInstanceParams.x);
#else
	return float(1.0);
#endif
}
 
MaterialFloat GetDistanceCullFade()
{
#if PIXELSHADER && !IS_NANITE_PASS // Nanite doesn't bind this UB
	return saturate(ResolvedView.RealTime * PrimitiveFade.FadeTimeScaleBias.x + PrimitiveFade.FadeTimeScaleBias.y);
#else
    return 1.0f;
#endif
}

MaterialFloat4 GetFontSignedDistanceData(FMaterialPixelParameters Parameters)
{
#if SHADER_TYPE == ST_GrayscaleFont || SHADER_TYPE == ST_ColorFont || SHADER_TYPE == ST_SdfFont || SHADER_TYPE == ST_MsdfFont || SHADER_TYPE == ST_Dynamic
	return Parameters.FontSignedDistanceData;
#else
	return float4(0.0f, 0.0f, 0.0f, 0.0f);
#endif
}

/** Rotates Position about the given axis by the given angle, in radians, and returns the offset to Position. */
float3 RotateAboutAxis(float4 NormalizedRotationAxisAndAngle, float3 PositionOnAxis, float3 Position)
{
	// Project Position onto the rotation axis and find the closest point on the axis to Position
	float3 ClosestPointOnAxis = PositionOnAxis + NormalizedRotationAxisAndAngle.xyz * dot(NormalizedRotationAxisAndAngle.xyz, Position - PositionOnAxis);
	// Construct orthogonal axes in the plane of the rotation
	float3 UAxis = Position - ClosestPointOnAxis;
	float3 VAxis = cross(NormalizedRotationAxisAndAngle.xyz, UAxis);
	float CosAngle;
	float SinAngle;
	sincos(NormalizedRotationAxisAndAngle.w, SinAngle, CosAngle);
	// Rotate using the orthogonal axes
	float3 R = UAxis * CosAngle + VAxis * SinAngle;
	// Reconstruct the rotated world space position
	float3 RotatedPosition = ClosestPointOnAxis + R;
	// Convert from position to a position offset
	return RotatedPosition - Position;
}

/**
 * Rotates Position about the given axis by the given angle, in radians, and returns the offset to Position.
 * Note that this returns an *offset*, so even though inputs are in LWC-space, the offset is returned as a regular float
 */
float3 RotateAboutAxis(float4 NormalizedRotationAxisAndAngle, FWSVector3 PositionOnAxis, FWSVector3 Position)
{
	// Project Position onto the rotation axis and find the closest point on the axis to Position
	FWSVector3 ClosestPointOnAxis = WSAdd(PositionOnAxis, NormalizedRotationAxisAndAngle.xyz * dot(NormalizedRotationAxisAndAngle.xyz, WSSubtractDemote(Position, PositionOnAxis)));
	// Construct orthogonal axes in the plane of the rotation
	float3 UAxis = WSSubtractDemote(Position, ClosestPointOnAxis);
	float3 VAxis = cross(NormalizedRotationAxisAndAngle.xyz, UAxis);
	float CosAngle;
	float SinAngle;
	sincos(NormalizedRotationAxisAndAngle.w, SinAngle, CosAngle);
	// Rotate using the orthogonal axes
	float3 R = UAxis * CosAngle + VAxis * SinAngle;

	// Here we want to compute the following values:
	// FLWCVector3 RotatedPosition = LWCAdd(ClosestPointOnAxis, R);
	// return LWCSubtract(RotatedPosition, Position);
	// This can logically be written like this:
	// return ClosestPointOnAxis + R - Position
	// Notice that UAxis is already defined as (Position - ClosestPointOnAxis)
	// So we can simply this as R - UAxis, to avoid some conversions to/from LWC
	return R - UAxis;
}

float ViewScalarBlueNoise(uint2 InScreenCoord, uint InFrameIndex)
{
	return GetBlueNoiseScalar(View.BlueNoiseScalarTexture, View.BlueNoiseDimensions, View.BlueNoiseModuloMasks, InScreenCoord, InFrameIndex);
}

float GetDitheredCoverage(uint2 InScreenCoord, uint InFrameIndex, float InCoverage)
{
	return GetBlueNoiseDitheredCoverage(View.BlueNoiseScalarTexture, View.BlueNoiseDimensions, View.BlueNoiseModuloMasks, InScreenCoord, InFrameIndex, InCoverage);
}

// Material Expression function
float MaterialExpressionDepthOfFieldFunction(float SceneDepth, int FunctionValueIndex)
{
	// tryed switch() but seems that doesn't work

	if (FunctionValueIndex == 0) // TDOF_NearAndFarMask
	{
		return CalcUnfocusedPercentCustomBound(SceneDepth, 1, 1);
	}
	else if(FunctionValueIndex == 1) // TDOF_Near
	{
		return CalcUnfocusedPercentCustomBound(SceneDepth, 1, 0);
	}
	else if(FunctionValueIndex == 2) // TDOF_Far
	{
		return CalcUnfocusedPercentCustomBound(SceneDepth, 0, 1);
	}
	else if(FunctionValueIndex == 3) // TDOF_CircleOfConfusionRadius
	{
		// * 2 to compensate for half res
		return DepthToCoc(SceneDepth) * 2.0f;
	}
	return 0;
}

// TODO convert to LUT
float3 MaterialExpressionBlackBody( float Temp )
{
	// Technically this approximation to the Planckian locus is only valid within the 1000-15000 temperature range, 
	// but we need to support temperatures outside this range too; especially the range between 0 and 1000.

	float u = ( 0.860117757f + 1.54118254e-4f * Temp + 1.28641212e-7f * Temp*Temp ) / ( 1.0f + 8.42420235e-4f * Temp + 7.08145163e-7f * Temp*Temp );
	float v = ( 0.317398726f + 4.22806245e-5f * Temp + 4.20481691e-8f * Temp*Temp ) / ( 1.0f - 2.89741816e-5f * Temp + 1.61456053e-7f * Temp*Temp );

	float x = 3*u / ( 2*u - 8*v + 4 );
	float y = 2*v / ( 2*u - 8*v + 4 );
	float z = 1 - x - y;

	float Y = 1;
	float X = Y/y * x;
	float Z = Y/y * z;

	// The XYZtoRGB transform can result in negative values, so we need to clamp here.
	return max( 0.0f, XYZ_2_LinearRGB( float3( X, Y, Z ) ) ) * pow( 0.0004 * Temp, 4 );
}

float2 MaterialExpressionGetHairRootUV(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsRootUV(Parameters.HairControlPointId);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsRootUV(Parameters.HairPrimitiveUV, Parameters.HairPrimitiveRootUV);
#else
	return float2(0, 0);
#endif
}

float2 MaterialExpressionGetHairUV(FMaterialPixelParameters Parameters)
{	
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsUV(Parameters.HairControlPointId, Parameters.HairPrimitiveUV);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsUV(Parameters.HairPrimitiveUV);
#else
	return float2(0,0);
#endif
}

float2 MaterialExpressionGetHairDimensions(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsDimensions(Parameters.HairControlPointId, Parameters.HairPrimitiveUV.x);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsDimensions(Parameters.HairPrimitiveUV, Parameters.HairPrimitiveLength);
#else
	return float2(0, 0);
#endif
}

float MaterialExpressionGetHairSeed(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsSeed(Parameters.HairControlPointId);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsSeed(Parameters.HairPrimitiveUV);
#else
	return 0;
#endif
}

float3 MaterialExpressionGetHairClumpID(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsClumpID(Parameters.HairControlPointId);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsClumpID(Parameters.HairPrimitiveUV);
#else
	return 0;
#endif
}

float3 MaterialExpressionGetHairBaseColor(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsColor(Parameters.HairControlPointId, Parameters.HairPrimitiveUV.x);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsColor(Parameters.HairPrimitiveUV, Parameters.HairPrimitiveMaterial.xyz);
#else
	return float3(0,0,0);
#endif
}

float MaterialExpressionGetHairRoughness(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsRoughness(Parameters.HairControlPointId, Parameters.HairPrimitiveUV.x);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsRoughness(Parameters.HairPrimitiveUV, Parameters.HairPrimitiveMaterial.w);
#else
	return 0;
#endif
}

float MaterialExpressionGetHairAO(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsAO(Parameters.HairControlPointId, Parameters.HairPrimitiveUV.x);
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsAO(Parameters.HairPrimitiveUV, 1.0f);
#else
	return 0;
#endif
}

float MaterialExpressionGetHairDepth(FMaterialVertexParameters Parameters)
{
	return 0;
}

float MaterialExpressionGetHairDepth(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsDepth();
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsDepth(Parameters.HairPrimitiveUV, Parameters.SvPosition.z);
#else
	return 0; 
#endif
}

float MaterialExpressionGetHairCoverage(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsCoverage();
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsCoverage(Parameters.HairPrimitiveUV);
#else
	return 0;
#endif
}

float3 MaterialExpressionGetHairTangent(FMaterialPixelParameters Parameters, bool bUseTangentSpace)
{
#if HAIR_STRAND_MESH_FACTORY
	return bUseTangentSpace ? float3(0,1,0) : Parameters.TangentToWorld[2];
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsTangent(Parameters.HairPrimitiveUV, Parameters.TangentToWorld, bUseTangentSpace);
#else
	return 0;
#endif	
}

float2 MaterialExpressionGetAtlasUVs(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return float2(0,0);
#elif HAIR_CARD_MESH_FACTORY
	return Parameters.HairPrimitiveUV;
#else
	return 0;
#endif

}

float4 MaterialExpressionGetHairAuxilaryData(FMaterialPixelParameters Parameters)
{
#if HAIR_CARD_MESH_FACTORY
	return GetHairCardsAuxilaryData(Parameters.HairPrimitiveUV);
#else
	return 0;
#endif
}

float MaterialExpressionGetHairGroupIndex(FMaterialPixelParameters Parameters)
{
#if HAIR_STRAND_MESH_FACTORY
	return GetHairStrandsGroupIndex();
#elif HAIR_CARD_MESH_FACTORY
	return GetHairCardsGroupIndex(Parameters.HairPrimitiveUV, Parameters.HairPrimitiveGroupIndex);
#else
	return 0;
#endif

}
float3 MaterialExpressionGetHairColorFromMelanin(float Melanin, float Redness, float3 DyeColor)
{
	return GetHairColorFromMelanin(Melanin, Redness, DyeColor);
}

#define DEFINE_ATMOSPHERELIGHTVECTOR(MaterialParamType) float3 MaterialExpressionAtmosphericLightVector(MaterialParamType Parameters) {return ResolvedView.AtmosphereLightDirection[0].xyz;}
DEFINE_ATMOSPHERELIGHTVECTOR(FMaterialPixelParameters)
DEFINE_ATMOSPHERELIGHTVECTOR(FMaterialVertexParameters)

float3 MaterialExpressionSkyAtmosphereLightIlluminanceOnGround(FMaterialPixelParameters Parameters, uint LightIndex)
{
#if PROJECT_SUPPORT_SKY_ATMOSPHERE // can be used anywhere, not only in sky material with MATERIAL_SKY_ATMOSPHERE
	return ResolvedView.AtmosphereLightIlluminanceOnGroundPostTransmittance[LightIndex].rgb;
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif
}

float3 MaterialExpressionAtmosphericLightColor(FMaterialPixelParameters Parameters) // Deprecated but back compatible.
{
	return MaterialExpressionSkyAtmosphereLightIlluminanceOnGround(Parameters, 0);
}

float3 MaterialExpressionSkyAtmosphereLightIlluminance(FMaterialPixelParameters Parameters, float3 TranslatedWorldPosition, uint LightIndex)
{
#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	const float3 PlanetCenterToTranslatedWorldPos = (TranslatedWorldPosition - ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;

	// GetAtmosphereTransmittance does a shadow test against the virtual planet.
	const float3 TransmittanceToLight = GetAtmosphereTransmittance(
		PlanetCenterToTranslatedWorldPos, ResolvedView.AtmosphereLightDirection[LightIndex].xyz, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
		View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);

	return ResolvedView.AtmosphereLightIlluminanceOuterSpace[LightIndex].rgb * TransmittanceToLight;
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif
}

float3 MaterialExpressionSkyAtmosphereLightIlluminance(FMaterialPixelParameters Parameters, FWSVector3 WorldPosition, uint LightIndex)
{
	const float3 TranslatedWorldPosition = DFFastToTranslatedWorld(WSToDF(WorldPosition), ResolvedView.PreViewTranslation);
	return MaterialExpressionSkyAtmosphereLightIlluminance(Parameters, TranslatedWorldPosition, LightIndex);
}

#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	#define DEFINE_SKYATMLIGHTDIRECTION(MaterialParamType) float3 MaterialExpressionSkyAtmosphereLightDirection(MaterialParamType Parameters, uint LightIndex) {return ResolvedView.AtmosphereLightDirection[LightIndex].xyz;}
#else
	#define DEFINE_SKYATMLIGHTDIRECTION(MaterialParamType) float3 MaterialExpressionSkyAtmosphereLightDirection(MaterialParamType Parameters, uint LightIndex) {return float3(0.0f, 0.0f, 0.0f);}
#endif
DEFINE_SKYATMLIGHTDIRECTION(FMaterialPixelParameters)
DEFINE_SKYATMLIGHTDIRECTION(FMaterialVertexParameters)

float3 MaterialExpressionSkyAtmosphereLightDiskLuminance(FMaterialPixelParameters Parameters, uint LightIndex, float OverrideAtmosphereLightDiscCosHalfApexAngle)
{
	float3 LightDiskLuminance = float3(0.0f, 0.0f, 0.0f);
#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	if (ResolvedView.RenderingReflectionCaptureMask == 0.0f) // Do not render light disk when in reflection capture in order to avoid double specular. The sun contribution is already computed analyticaly.
	{
		const float3 ViewDir = -Parameters.CameraVector;
		float3 PlanetCenterToCameraTranslatedWorld = (ResolvedView.SkyCameraTranslatedWorldOrigin - ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz);
		if (IsOrthoProjection())
		{
			PlanetCenterToCameraTranslatedWorld = GetTranslatedWorldCameraPosFromView(Parameters.SvPosition.xy, true) + PlanetCenterToCameraTranslatedWorld;
		}
		PlanetCenterToCameraTranslatedWorld *= CM_TO_SKY_UNIT;

		// GetLightDiskLuminance does a test against the virtual planet but SkyCameraTranslatedWorldOrigin is always put safely setup above it (to never have the camera into the virtual planet with a black screen)
		LightDiskLuminance =  GetLightDiskLuminance(PlanetCenterToCameraTranslatedWorld, ViewDir, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
			View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler,
			ResolvedView.AtmosphereLightDirection[LightIndex].xyz, 
			OverrideAtmosphereLightDiscCosHalfApexAngle >= 0.0f ? OverrideAtmosphereLightDiscCosHalfApexAngle : ResolvedView.AtmosphereLightDiscCosHalfApexAngle_PPTrans[LightIndex].x,
			ResolvedView.AtmosphereLightDiscLuminance[LightIndex].xyz);
	}
#endif
	return LightDiskLuminance;
}

float3 MaterialExpressionSkyAtmosphereViewLuminance(FMaterialPixelParameters Parameters, float3 ViewDir)
{
#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	if (ResolvedView.RenderingReflectionCaptureMask == 0.0f && !IsSkyAtmosphereRenderedInMain(ResolvedView.EnvironmentComponentsFlags))
	{
		return float3(0.0f, 0.0f, 0.0f);
	}

	// The referencial used to build the Sky View lut
	float3x3 LocalReferencial = GetSkyViewLutReferential(ResolvedView.SkyViewLutReferential);

	float ViewHeight = ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.w;
	if(IsOrthoProjection())
	{
		float3 CameraPosition = GetTranslatedWorldCameraPosFromView(Parameters.SvPosition.xy, true);
		CameraPosition.z += ViewHeight;
		ViewHeight = mul(LocalReferencial, CameraPosition).z;
	}
	ViewHeight *= CM_TO_SKY_UNIT;

	// Compute inputs in this referential
	float3 WorldPosLocal = float3(0.0, 0.0, ViewHeight);
	float3 UpVectorLocal = float3(0.0, 0.0, 1.0);
	float3 WorldDirLocal = mul(LocalReferencial, ViewDir);
	float ViewZenithCosAngle = dot(WorldDirLocal, UpVectorLocal);

	float2 Sol = RayIntersectSphere(WorldPosLocal, WorldDirLocal, float4(0.0f, 0.0f, 0.0f, ResolvedView.SkyAtmosphereBottomRadiusKm));
	const bool IntersectGround = any(Sol > 0.0f);

	float2 SkyViewLutUv;
	SkyViewLutParamsToUv(IntersectGround, ViewZenithCosAngle, WorldDirLocal, ViewHeight, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyViewLutSizeAndInvSize, SkyViewLutUv);
	float3 SkyAtmosphereViewLuminance = Texture2DSampleLevel(View.SkyViewLutTexture, View.SkyViewLutTextureSampler, SkyViewLutUv, 0.0f).rgb;
	SkyAtmosphereViewLuminance *= ResolvedView.SkyAtmosphereSkyLuminanceFactor.rgb;
	SkyAtmosphereViewLuminance *= ResolvedView.OneOverPreExposure;

	if (ResolvedView.RealTimeReflectionCapture == 0.0f)
	{
		SkyAtmosphereViewLuminance.rgb = QuantizeFloatColor(SkyAtmosphereViewLuminance.rgb, ResolvedView.SceneColorTextureFormatQuantizationError, Parameters.SvPosition.xy, QUANTIZE_NOISE_INTERLEAVEDGRADIENT);
	}

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags) && View.bPrimitiveAlphaHoldoutEnabled)
	{
		SkyAtmosphereViewLuminance = 0.0f; // Disable sky dome luminance contribution. Throughput will be 0.
	}
#endif
	
	return SkyAtmosphereViewLuminance;
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif
}

float4 MaterialExpressionSkyAtmosphereAerialPerspective(FMaterialPixelParameters Parameters, float3 TranslatedWorldPosition)
{
#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	const float3 SkyTranslatedWorldPosition = TranslatedWorldPosition * CM_TO_SKY_UNIT;
	const float3 SkyCameraTranslatedWorldOrigin = ResolvedView.SkyCameraTranslatedWorldOrigin.xyz * CM_TO_SKY_UNIT;

	// NDCPosition is not computed using WorldPosition because it could result in position outside the frustum, 
	// distorted uvs and bad visuals with artefact. Only the distance computation can actually be benefit from the surface position specified here.
	float4 NDCPosition = mul(float4(GetTranslatedWorldPosition(Parameters), 1.0f), ResolvedView.TranslatedWorldToClip);

	const float NearFadeOutRangeInvDepthKm = 1.0 / 0.00001f; // 1 centimeter fade region
	float4 AerialPerspective = GetAerialPerspectiveLuminanceTransmittance(
		ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
		NDCPosition, SkyTranslatedWorldPosition - SkyCameraTranslatedWorldOrigin,
		View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler,
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
		ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm,
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
		ResolvedView.OneOverPreExposure,
		NearFadeOutRangeInvDepthKm);
	
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags) && View.bPrimitiveAlphaHoldoutEnabled)
	{
		AerialPerspective.rgb = 0.0f;
	}
#endif
	
	return AerialPerspective;
#else
	return float4(0.0f, 0.0f, 0.0f, 1.0f); // RGB= null scattering, A= null transmittance
#endif
}

float4 MaterialExpressionSkyAtmosphereAerialPerspective(FMaterialPixelParameters Parameters, FWSVector3 WorldPosition)
{
	const float3 TranslatedWorldPosition = WSAddDemote(WorldPosition, Parameters.LWCData.PreViewTranslation);
	return MaterialExpressionSkyAtmosphereAerialPerspective(Parameters, TranslatedWorldPosition);
}

float3 MaterialExpressionSkyAtmosphereDistantLightScatteredLuminance(FMaterialPixelParameters Parameters)
{
#if MATERIAL_SKY_ATMOSPHERE && PROJECT_SUPPORT_SKY_ATMOSPHERE
	return GetViewDistanceSkyLightColor();
#else
	return float3(0.0f, 0.0f, 0.0f);
#endif
}

// Forward declaration for function defined in BasePassCommonFunctions.ush relying on ForwardLightStruct.
// This is needed because BasePassCommon.ush is dependent on MaterialTemplate.ush, so it must included after, but those functions needs to know about ForwardLightStruct.
float3 MaterialExpressionMainDirectionalLightIlluminance();
float3 MaterialExpressionMainDirectionalLightDirection();

#if POST_PROCESS_MATERIAL
uint bSceneDepthWithoutWaterTextureAvailable;
Texture2D SceneDepthWithoutSingleLayerWaterTexture;
SamplerState SceneDepthWithoutSingleLayerWaterSampler;
float4 SceneWithoutSingleLayerWaterMinMaxUV;
float2 SceneWithoutSingleLayerWaterTextureSize;
float2 SceneWithoutSingleLayerWaterInvTextureSize;
#endif

/**
	Get the scene depth from the scene below the single layer water surface. This is only valid in the single layer water rendering pass or during post processing.
	Returns the chosen fallback depth if the material doesn't support reading back the correct depth.
*/
float MaterialExpressionSceneDepthWithoutWater(float2 ViewportUV, float FallbackDepth)
{
#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER && !HIT_PROXY_SHADER && !SCENE_TEXTURES_DISABLED && (SINGLE_LAYER_WATER_SHADING_QUALITY != SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_BUFFER)

#if SHADING_PATH_MOBILE
	#if MOBILE_SINGLELAYERWATER_FALLBACK
		return FallbackDepth;
	#else
		const float4 SLWBufferBilinearUVMinMax = View.BufferSizeAndInvSize.zwzw * (ResolvedView.ViewRectMinAndSize.xyxy + float4(0.55f, 0.55f, ResolvedView.ViewRectMinAndSize.zw - 0.55f));
		const float2 ClampedViewportUV = clamp(ViewportUV, SLWBufferBilinearUVMinMax.xy, SLWBufferBilinearUVMinMax.zw);
		float WaterDeviceZ = WaterSampleSceneDepthWithoutWater(MobileBasePass.SceneDepthCopyTexture, MobileBasePass.SceneDepthCopySampler, ClampedViewportUV, View.BufferSizeAndInvSize.xy, View.BufferSizeAndInvSize.zw);
	#endif
#else
	const float4 SLWRefractionMinMaxUV = GetSingleLayerWaterRefractionUVMinMax(SingleLayerWater.SceneWithoutSingleLayerWaterTextureSize, SingleLayerWater.SceneWithoutSingleLayerWaterInvTextureSize);
	const float2 ClampedViewportUV = clamp(ViewportUV, SLWRefractionMinMaxUV.xy, SLWRefractionMinMaxUV.zw);
	float WaterDeviceZ = WaterSampleSceneDepthWithoutWater(SingleLayerWater.SceneDepthWithoutSingleLayerWaterTexture, SingleLayerWaterSceneDepthSampler, ClampedViewportUV,
		SingleLayerWater.SceneWithoutSingleLayerWaterTextureSize, SingleLayerWater.SceneWithoutSingleLayerWaterInvTextureSize);
#endif
	return ConvertFromDeviceZ(WaterDeviceZ);

#elif POST_PROCESS_MATERIAL
	if (bSceneDepthWithoutWaterTextureAvailable)
	{
		const float2 ClampedViewportUV = clamp(ViewportUV, SceneWithoutSingleLayerWaterMinMaxUV.xy, SceneWithoutSingleLayerWaterMinMaxUV.zw);
		float WaterDeviceZ = WaterSampleSceneDepthWithoutWater(SceneDepthWithoutSingleLayerWaterTexture, SceneDepthWithoutSingleLayerWaterSampler, ClampedViewportUV,
			SceneWithoutSingleLayerWaterTextureSize, SceneWithoutSingleLayerWaterInvTextureSize);
		return ConvertFromDeviceZ(WaterDeviceZ);
	}
	else
	{
#if SHADING_PATH_DEFERRED
		return CalcSceneDepth(uint2(ViewportUV * View.BufferSizeAndInvSize.xy));
#elif SHADING_PATH_MOBILE
		return CalcSceneDepth(ViewportUV);
#else
		return FallbackDepth;
#endif
	}
#else
	return FallbackDepth;
#endif
}

float MaterialExpressionCloudSampleAltitude(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.CloudSampleAltitude;
#else
	return 0.0f;
#endif
}

float MaterialExpressionCloudSampleAltitudeInLayer(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.CloudSampleAltitudeInLayer;
#else
	return 0.0f;
#endif
}

float MaterialExpressionCloudSampleNormAltitudeInLayer(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.CloudSampleNormAltitudeInLayer;
#else
	return 0.0f;
#endif
}

float4 MaterialExpressionVolumeSampleConservativeDensity(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.VolumeSampleConservativeDensity;
#else
	return float4(0.0f, 0.0f, 0.0f, 0.0f);
#endif
}

float MaterialExpressionVolumeSampleShadowSampleDistance(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.ShadowSampleDistance;
#else
	return 0.0f;
#endif
}

float3 MaterialExpressionCloudEmptySpaceSkippingSphereCenterWorldPosition(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.CloudEmptySpaceSkippingSphereCenterWorldPosition;
#else
	return 0.0f;
#endif
}

float MaterialExpressionCloudEmptySpaceSkippingSphereRadius(FMaterialPixelParameters Parameters)
{
#if CLOUD_LAYER_PIXEL_SHADER
	return Parameters.CloudEmptySpaceSkippingSphereRadius;
#else
	return 0.0f;
#endif
}

// We use a forward declaration towards the real implementation of functions we need. This is always defined for base passes (opaque and translucent)
#if defined(IS_BASE_PASS) && IS_MOBILE_BASE_PASS==0
float3 GetSkyLightReflectionSupportingBlend(float3 ReflectionVector, float Roughness, out float OutSkyAverageBrightness);
#elif defined(IS_BASE_PASS) && IS_MOBILE_BASE_PASS==1
half3 GetMobileSkyLightReflection(half3 ReflectionVector, half Roughness, half CubemapMaxMip);
#endif

float3 MaterialExpressionSkyLightEnvMapSample(float3 Direction, float Roughness)
{
#if defined(IS_BASE_PASS) && IS_MOBILE_BASE_PASS==0
	float SkyAverageBrightness = 1.0f;
	return GetSkyLightReflectionSupportingBlend(Direction, Roughness, SkyAverageBrightness);
#elif defined(IS_BASE_PASS) && IS_MOBILE_BASE_PASS==1
	return GetMobileSkyLightReflection(Direction, Roughness, MobileReflectionCapture.Params.y);
#else 
	return 0.0f;
#endif
}

/**
 * Utility function to unmirror one coordinate value to the other side
 * UnMirrored == 1 if normal
 * UnMirrored == -1 if mirrored
 *
 * Used by most of parameter functions generated via code in this file
 */
MaterialFloat UnMirror( MaterialFloat Coordinate, FMaterialPixelParameters Parameters )
{
	return ((Coordinate)*(Parameters.UnMirrored)*0.5+0.5);
}

/**
 * UnMirror only U
 */
MaterialFloat2 UnMirrorU( MaterialFloat2 UV, FMaterialPixelParameters Parameters )
{
	return MaterialFloat2(UnMirror(UV.x, Parameters), UV.y);
}

/**
 * UnMirror only V
 */
MaterialFloat2 UnMirrorV( MaterialFloat2 UV, FMaterialPixelParameters Parameters )
{
	return MaterialFloat2(UV.x, UnMirror(UV.y, Parameters));
}

/**
 * UnMirror only UV
 */
MaterialFloat2 UnMirrorUV( MaterialFloat2 UV, FMaterialPixelParameters Parameters )
{
	return MaterialFloat2(UnMirror(UV.x, Parameters), UnMirror(UV.y, Parameters));
}

/** 
 * Transforms screen space positions into UVs with [.5, .5] centered on ObjectPostProjectionPosition,
 * And [1, 1] at ObjectPostProjectionPosition + (ObjectRadius, ObjectRadius).
 */
MaterialFloat2 GetParticleMacroUV(FMaterialPixelParameters Parameters)
{
	return (Parameters.ScreenPosition.xy / Parameters.ScreenPosition.w - Parameters.Particle.MacroUV.xy) * Parameters.Particle.MacroUV.zw + MaterialFloat2(.5, .5);
}

/** Accesses a shared material sampler or falls back if independent samplers are not supported. */
SamplerState GetMaterialSharedSampler(SamplerState TextureSampler, SamplerState SharedSampler)
{
#if SUPPORTS_INDEPENDENT_SAMPLERS
	return SharedSampler;
#else
	// Note: to match behavior on platforms that don't support SUPPORTS_INDEPENDENT_SAMPLERS, 
	// TextureSampler should have been set to the same sampler.  This is not currently done.
	return TextureSampler;
#endif
}

/** Calculate a reflection vector about the specified world space normal. Optionally normalize this normal **/
MaterialFloat3 ReflectionAboutCustomWorldNormal(FMaterialPixelParameters Parameters, MaterialFloat3 WorldNormal, bool bNormalizeInputNormal)
{
	if (bNormalizeInputNormal)
	{
		WorldNormal = normalize(WorldNormal);
	}

	return -Parameters.CameraVector + WorldNormal * dot(WorldNormal, Parameters.CameraVector) * 2.0;
}

#ifndef SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS
#define SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS 0
#endif

/** 
 * Calculates opacity for a billboard particle as if it were a sphere. 
 * Note: Calling this function requires the vertex factory to have been compiled with SPHERICAL_PARTICLE_OPACITY set to 1
 */
float GetSphericalParticleOpacity(FMaterialPixelParameters Parameters, float Density)
{
	float Opacity = 0;

#if PARTICLE_FACTORY || HAS_PRIMITIVE_UNIFORM_BUFFER

#if PARTICLE_FACTORY

	float3 ParticleTranslatedWorldPosition = Parameters.Particle.TranslatedWorldPositionAndSize.xyz;
	float ParticleRadius = max(0.000001f, Parameters.Particle.TranslatedWorldPositionAndSize.w);

#elif HAS_PRIMITIVE_UNIFORM_BUFFER

	// Substitute object attributes if the mesh is not a particle
	// This is mostly useful for previewing materials using spherical opacity in the material editor
	float3 ParticleTranslatedWorldPosition = GetObjectTranslatedWorldPosition(Parameters);
	float ParticleRadius = max(0.000001f, GetPrimitiveData(Parameters).ObjectRadius);

#endif

	// Rescale density to make the final opacity independent of the particle radius
	float RescaledDensity = Density / ParticleRadius;

	// Distance from point being shaded to particle center
	float DistanceToParticle = length(Parameters.WorldPosition_NoOffsets_CamRelative - ParticleTranslatedWorldPosition);

	FLATTEN
	if (DistanceToParticle < ParticleRadius) 
	{
		// Distance from point being shaded to the point on the sphere along the view direction
		float HemisphericalDistance = sqrt(ParticleRadius * ParticleRadius - DistanceToParticle * DistanceToParticle);

#if SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS
		// When rendering shadow depths we can't use scene depth or the near plane, just use the distance through the whole sphere
		float DistanceThroughSphere = HemisphericalDistance * 2;
#else
		// Initialize near and far sphere intersection distances
		float NearDistance = Parameters.ScreenPosition.w - HemisphericalDistance;
		float FarDistance = Parameters.ScreenPosition.w + HemisphericalDistance;

		float SceneDepth = CalcSceneDepth(SvPositionToBufferUV(Parameters.SvPosition));
		FarDistance = min(SceneDepth, FarDistance);

		// Take into account opaque objects intersecting the sphere
		float DistanceThroughSphere = FarDistance - NearDistance;
#endif

		// Use the approximation for the extinction line integral from "Spherical Billboards and their Application to Rendering Explosions"
		Opacity = saturate(1 - exp2(-RescaledDensity * (1 - DistanceToParticle / ParticleRadius) * DistanceThroughSphere));

#if !SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS
		// Fade out as the particle approaches the near plane
		Opacity = lerp(0, Opacity, saturate((Parameters.ScreenPosition.w - ParticleRadius - ResolvedView.NearPlane) / ParticleRadius));
#endif
	}

#endif

	return Opacity;
}

#define LWCADDRESSMODE_CLAMP 0u
#define LWCADDRESSMODE_WRAP 1u
#define LWCADDRESSMODE_MIRROR 2u

float LWCApplyAddressModeWrap(FLWCScalar V)
{
	// Compute the fractional part of the tile, then add to the offset
	// Let the texture unit apply the final coordinate wrapping, which will allow derivatives to work correctly unless we cross a tile boundary
	const float FracTile = frac(LWCGetTile(V) * UE_LWC_RENDER_TILE_SIZE);
	return FracTile + V.Offset;
}

float LWCApplyAddressModeMirror(FLWCScalar v)
{
	// Unclear what the best option is for MIRROR
	// We can apply the mirror logic directly, but that will break derivatives
	// We can use similar logic as WRAP, but in that case results will actually be incorrect at tile boundaries (not just wrong derivatives)
	// Or we can convert to float and accept precision loss for large values (but otherwise correct)
	// TODO - something better?

	//float t = LWCFrac(LWCMultiply(v, 0.5f)) * 2.0f;
	//return 1.0f - abs(t - 1.0f);
	return LWCToFloat(v);
}

float LWCApplyAddressModeClamp(FLWCScalar v)
{
	// For the CLAMP case, a simple LWCToFloat() is sufficient.  This will lose a ton of precision for large values, but we don't care about this since the GPU will clamp to [0,1) anyway
	// It's possible certain GPUs might need a special case if the ToFloat() conversion overflows
	return LWCToFloat(v);
}

float LWCApplyAddressMode(FLWCScalar v, uint AddressMode)
{
	if(AddressMode == LWCADDRESSMODE_WRAP) return LWCApplyAddressModeWrap(v);
	else if(AddressMode == LWCADDRESSMODE_MIRROR) return LWCApplyAddressModeMirror(v);
	else return LWCApplyAddressModeClamp(v);
}
float2 LWCApplyAddressMode(FLWCVector2 UV, uint AddressX, uint AddressY)
{
	return float2(LWCApplyAddressMode(LWCGetX(UV), AddressX), LWCApplyAddressMode(LWCGetY(UV), AddressY));
}
float3 LWCApplyAddressMode(FLWCVector3 UV, uint AddressX, uint AddressY, uint AddressZ)
{
	return float3(LWCApplyAddressMode(LWCGetX(UV), AddressX), LWCApplyAddressMode(LWCGetY(UV), AddressY), LWCApplyAddressMode(LWCGetZ(UV), AddressZ));
}


float DFApplyAddressModeWrap(FDFScalar V)
{
	// Hacky and certainly not flawless, but reasonably fast.
	// The resulting value is not in [0;1] so the texture unit apply the final coordinate wrapping, which will allow derivatives to work correctly unless cross the modulo boundary.
	const float Divisor = 0x1000; // Chosen somewhat arbitrarily. Large enough to not have seams everywhere, but small enough to have plenty of fractional precision in the result.
	return DFFmodByPow2Demote(V, Divisor);
}

float DFApplyAddressModeMirror(FDFScalar V)
{
	// Unclear what the best option is for MIRROR
	// We can apply the mirror logic directly, but that will break derivatives
	// We can use similar logic as WRAP, but in that case results will actually be incorrect at tile boundaries (not just wrong derivatives)
	// Or we can convert to float and accept precision loss for large values (but otherwise correct)
	// TODO - something better?

	float a = DFFracDemote(DFDivideByPow2(V, 2));
	float b = 2.0 * a;
	float c = 2.0 * (1.0 - a);
	return min(b, c); //DF_TODO: derivatives? Maybe use the same logic from Wrap instead?
}

float DFApplyAddressModeClamp(FDFScalar V)
{
	// For the CLAMP case, a simple DFDemote() is sufficient.  This will lose a ton of precision for large values, but we don't care about this since the GPU will clamp to [0,1) anyway
	// It's possible certain GPUs might need a special case if the ToFloat() conversion overflows
	return DFDemote(V);
}

float DFApplyAddressMode(FDFScalar v, uint AddressMode)
{
	if(AddressMode == LWCADDRESSMODE_WRAP) return DFApplyAddressModeWrap(v);
	else if(AddressMode == LWCADDRESSMODE_MIRROR) return DFApplyAddressModeMirror(v);
	else return DFApplyAddressModeClamp(v);
}
float2 DFApplyAddressMode(FDFVector2 UV, uint AddressX, uint AddressY)
{
	return float2(DFApplyAddressMode(DFGetX(UV), AddressX), DFApplyAddressMode(DFGetY(UV), AddressY));
}
float3 DFApplyAddressMode(FDFVector3 UV, uint AddressX, uint AddressY, uint AddressZ)
{
	return float3(DFApplyAddressMode(DFGetX(UV), AddressX), DFApplyAddressMode(DFGetY(UV), AddressY), DFApplyAddressMode(DFGetZ(UV), AddressZ));
}

#if WSVECTOR_IS_TILEOFFSET
#define WSApplyAddressMode LWCApplyAddressMode
#elif WSVECTOR_IS_DOUBLEFLOAT
#define WSApplyAddressMode DFApplyAddressMode
#endif

float2 RotateScaleOffsetTexCoords(float2 InTexCoords, float4 InRotationScale, float2 InOffset)
{
	return float2(dot(InTexCoords, InRotationScale.xy), dot(InTexCoords, InRotationScale.zw)) + InOffset;
}

FWSVector2 RotateScaleOffsetTexCoords(FWSVector2 InTexCoords, float4 InRotationScale, float2 InOffset)
{
	return WSAdd(MakeWSVector(WSDot(InTexCoords, InRotationScale.xy), WSDot(InTexCoords, InRotationScale.zw)), InOffset);
}

#if USES_SPEEDTREE

/** Vertex offset for SpeedTree wind and LOD */
float3 GetSpeedTreeVertexOffsetInner(FMaterialVertexParameters Parameters, int GeometryType, int WindType, int LODType, float BillboardThreshold, bool bExtraBend, float3 ExtraBend, FSpeedTreeData STData) 
{
#if !VF_SUPPORTS_SPEEDTREE_WIND
	return 0;
#else
	#if (NUM_MATERIAL_TEXCOORDS_VERTEX < 6) || IS_MESHPARTICLE_FACTORY
		return float4(0,0,0);
	#endif

	#if USE_INSTANCING || USE_INSTANCE_CULLING
		float3x3 LocalToWorld = DFToFloat3x3(Parameters.InstanceLocalToWorld);
		float3 LocalPosition = Parameters.PositionInstanceSpace;

		// skip if this instance is hidden
		if (Parameters.PerInstanceParams.y < 1.f)
		{
			return float3(0,0,0);
		}
	#else
		float3x3 LocalToWorld = DFToFloat3x3(GetPrimitiveData(Parameters).LocalToWorld);
		float3 LocalPosition = WSMultiplyDemote(GetWorldPosition(Parameters), Parameters.LWCData.WorldToLocal);
	#endif

	FWSVector3 TreePos = GetObjectWorldPosition(Parameters);

	// compute LOD by finding screen space size
	float LodInterp = 1.0;
#if !USE_INSTANCING || !USE_DITHERED_LOD_TRANSITION
	if (LODType == SPEEDTREE_LOD_TYPE_SMOOTH) 
	{
		const float Dist = length(WSSubtractDemote(TreePos, DFToWS(ResolvedView.WorldCameraOrigin)));
        const float ScreenMultiple = 0.5 * max(ResolvedView.ViewToClip[0][0], ResolvedView.ViewToClip[1][1]);
        const float ScreenRadius = 2.0 * ScreenMultiple * GetPrimitiveData(Parameters).ObjectRadius / max(1.0, Dist);
        LodInterp = saturate((ScreenRadius - SpeedTreeLODInfo.x) / SpeedTreeLODInfo.z);
	}
#endif
	float3 TreePosOffset = WSHackToFloat(WSDivideByPow2(TreePos, 1024)); // The only other use of the tree position is as an offset into trig functions, but big numbers don't play nice there

	// SpeedTrees should only be uniformly scaled, but if necessary, it takes a few more instructions
	float TreeScale = length(mul(float3(0,0,1), LocalToWorld));
					//float3(length((float3)LocalToWorld[0]),
					//		length((float3)LocalToWorld[1]),
					//		length((float3)LocalToWorld[2]));


	// @todo There is probably a more optimal way to get the rotated (but not translated or scaled) vertex position needed for correct wind
	float3 OriginalPosition = LocalPosition;
	OriginalPosition = mul(OriginalPosition, LocalToWorld) / TreeScale;

	float3 FinalPosition = OriginalPosition;
	
	if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_BILLBOARD)
	{
		if (BillboardThreshold < 1.0)
		{
			// billboard meshes can have triangles drop out if they aren't facing the camera
			// this rotates the view direction around so we ignore the local Z component
			float3 LocalView2D = normalize(float3(ResolvedView.ViewForward.xy, 0));
			float3 LocalNormal2D = normalize(float3(Parameters.TangentToWorld[2].xy, 0));
			if (dot(LocalView2D, LocalNormal2D) > (-1.0 + BillboardThreshold * 0.25))
			{
				FinalPosition = float3(0,0,0);
			}
		}
	}
	else
	{
		// rotated normal needed in a few places
		float3 Normal = Parameters.TangentToWorld[2];

		// branches and fronds
		if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_BRANCH || GeometryType == SPEEDTREE_GEOMETRY_TYPE_FROND) 
		{
			// smooth LOD
			#if !USE_INSTANCING
				if (LODType == SPEEDTREE_LOD_TYPE_SMOOTH) 
				{
					float3 LODPos = float3(Parameters.TexCoords[3].x, Parameters.TexCoords[3].y, Parameters.TexCoords[4].x);
					LODPos = mul(LODPos, LocalToWorld) / TreeScale;
					FinalPosition = lerp(LODPos, FinalPosition, LodInterp);
				}
			#endif

			// frond wind, if needed
			if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_FROND && WindType == SPEEDTREE_WIND_TYPE_PALM)
			{
				float2 TexCoords = Parameters.TexCoords[0];
				float4 WindExtra = float4(Parameters.TexCoords[5].x, Parameters.TexCoords[5].y, Parameters.TexCoords[6].x, 0.0);
				FinalPosition = RippleFrond(STData, FinalPosition, Normal, TexCoords.x, TexCoords.y, WindExtra.x, WindExtra.y, WindExtra.z);
			}
		}

		// leaves and facing leaves
		if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_FACINGLEAF || 
				(GeometryType == SPEEDTREE_GEOMETRY_TYPE_LEAF && 
				(LODType == SPEEDTREE_LOD_TYPE_SMOOTH || (WindType > SPEEDTREE_WIND_TYPE_FASTEST && WindType != SPEEDTREE_WIND_TYPE_PALM))))
		{
			// remove anchor pos from vertex position
			float3 Anchor = float3(Parameters.TexCoords[4].y, Parameters.TexCoords[5].x, Parameters.TexCoords[5].y);

			// face camera-facing leaves to the camera, if needed
			if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_FACINGLEAF) 
			{
				// have to rotate the view into local space
				FinalPosition = LocalPosition - Anchor;
				FinalPosition = FinalPosition.x * ResolvedView.ViewRight + 
								FinalPosition.y * ResolvedView.ViewUp + 
								FinalPosition.z * ResolvedView.ViewForward;
			}
			
			Anchor = (mul(Anchor, LocalToWorld)) / TreeScale;
			
			if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_LEAF)
			{
				FinalPosition -= Anchor;
			}

			// smooth LOD
			#if !USE_INSTANCING
				if (LODType == SPEEDTREE_LOD_TYPE_SMOOTH) 
				{
					if (GeometryType == SPEEDTREE_GEOMETRY_TYPE_LEAF)
					{
						float3 LODPos = float3(Parameters.TexCoords[3].x, Parameters.TexCoords[3].y, Parameters.TexCoords[4].x);
						LODPos = mul(LODPos, LocalToWorld) / TreeScale - Anchor;
						FinalPosition = lerp(LODPos, FinalPosition, LodInterp);
					}
					else
					{
						float LODScalar = Parameters.TexCoords[3].x;
						FinalPosition *= lerp(LODScalar, 1.0, LodInterp);
					}
				}
			#endif

			// leaf wind
			if (WindType > SPEEDTREE_WIND_TYPE_FASTEST && WindType != SPEEDTREE_WIND_TYPE_PALM) 
			{
				float4 WindExtra = float4(Parameters.TexCoords[6].x, Parameters.TexCoords[6].y, Parameters.TexCoords[7].x, Parameters.TexCoords[7].y);
				float LeafWindTrigOffset = Anchor.x + Anchor.y;
				FinalPosition = LeafWind(STData, WindExtra.w > 0.0, FinalPosition, Normal, WindExtra.x, float3(0,0,0), WindExtra.y, WindExtra.z, LeafWindTrigOffset, WindType);
			}
				
			// move leaf back to anchor
			FinalPosition += Anchor;
		}

		if (WindType > SPEEDTREE_WIND_TYPE_FAST)
		{
			// branch wind (applies to all geometry)
			float2 VertBranchWind = Parameters.TexCoords[2];
			FinalPosition = BranchWind(STData, FinalPosition, TreePosOffset, float4(VertBranchWind, 0, 0), WindType);
		}	
	}

	// global wind can apply to the whole tree, even billboards
	bool bHasGlobal = (WindType != SPEEDTREE_WIND_TYPE_NONE);
	if (bExtraBend || bHasGlobal)
	{
		FinalPosition = GlobalWind(STData, FinalPosition, TreePosOffset, true, bHasGlobal, bExtraBend, ExtraBend);
	}

	// convert into a world space offset
	return (FinalPosition - OriginalPosition) * TreeScale;
#endif // !VF_SUPPORTS_SPEEDTREE_WIND
}

/** Vertex offset for SpeedTree wind and LOD */
float3 GetSpeedTreeVertexOffset(FMaterialVertexParameters Parameters, int GeometryType, int WindType, int LODType, float BillboardThreshold, bool bUsePreviousFrame, bool bExtraBend, float3 ExtraBend) 
{
	if (bUsePreviousFrame)
	{
		return GetSpeedTreeVertexOffsetInner(Parameters, GeometryType, WindType, LODType, BillboardThreshold, bExtraBend, ExtraBend, GetPreviousSpeedTreeData());
	}
	return GetSpeedTreeVertexOffsetInner(Parameters, GeometryType, WindType, LODType, BillboardThreshold, bExtraBend, ExtraBend, GetCurrentSpeedTreeData());
}

float3 GetSpeedTreeVertexOffset(FMaterialVertexParameters Parameters, int GeometryType, int WindType, int LODType, float BillboardThreshold, bool bExtraBend, float3 ExtraBend) 
{
	return GetSpeedTreeVertexOffsetInner(Parameters, GeometryType, WindType, LODType, BillboardThreshold, bExtraBend, ExtraBend, GetCurrentSpeedTreeData());
}

float3 GetPrevSpeedTreeVertexOffset(FMaterialVertexParameters Parameters, int GeometryType, int WindType, int LODType, float BillboardThreshold, bool bExtraBend, float3 ExtraBend) 
{
	return GetSpeedTreeVertexOffsetInner(Parameters, GeometryType, WindType, LODType, BillboardThreshold, bExtraBend, ExtraBend, GetPreviousSpeedTreeData());
}

#endif

MaterialFloat2 GetLightmapUVs(FMaterialPixelParameters Parameters)
{
#if LIGHTMAP_UV_ACCESS
	return Parameters.LightmapUVs;
#else
	return MaterialFloat2(0,0);
#endif
}

MaterialFloat2 GetLightmapUVs_DDX(FMaterialPixelParameters Parameters)
{
#if LIGHTMAP_UV_ACCESS
	return Parameters.LightmapUVs_DDX;
#else
	return MaterialFloat2(0, 0);
#endif
}

MaterialFloat2 GetLightmapUVs_DDY(FMaterialPixelParameters Parameters)
{
#if LIGHTMAP_UV_ACCESS
	return Parameters.LightmapUVs_DDY;
#else
	return MaterialFloat2(0, 0);
#endif
}

//The post-process material needs to decode the scene color since it's encoded at PreTonemapMSAA if MSAA enabled on MetalMobilePlatorm 
//The POST_PROCESS_MATERIAL_BEFORE_TONEMAP is 1 for both BL_SceneColorBeforeDOF and BL_SceneColorAfterDOF post-process materials
#if FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1 && POST_PROCESS_MATERIAL && POST_PROCESS_MATERIAL_BEFORE_TONEMAP && METAL_ES3_1_PROFILE
uint bMetalMSAAHDRDecode;
#endif

#if NEEDS_SCENE_TEXTURES

#if POST_PROCESS_MATERIAL
// SceneTextureId of UserSceneTexture SceneColor input, to allow SceneColor alpha to be dynamically propagated
uint UserSceneTextureSceneColorInput;
#endif

#if SHADING_PATH_MOBILE

MaterialFloat4 MobileSceneTextureLookup(inout FMaterialPixelParameters Parameters, int SceneTextureId, float2 UV)
{
#if SCENE_TEXTURES_DISABLED
	// When scene textures are disabled, the output is matched to the dummy scene texture defaults.
	switch(SceneTextureId)
	{
	case PPI_SceneDepth:
	case PPI_CustomDepth:
		return ConvertFromDeviceZ(0.0f);
	case PPI_MaterialAO:
	case PPI_CustomStencil:
		return 1.0f;
	default:
		return 0.0f;
	}
#else

	#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
	static float PropagatedAlpha = 0.0f;
	#endif

	const float2 PixelPos = UV * View.BufferSizeAndInvSize.xy;
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UV, PixelPos, GetViewId(Parameters));

	switch(SceneTextureId)
	{
		// order needs to match to ESceneTextureId
	case PPI_SceneDepth:
		return GBuffer.Depth;
	case PPI_DiffuseColor:
		return MaterialFloat4(GBuffer.DiffuseColor, 0);
	case PPI_SpecularColor:
		return MaterialFloat4(GBuffer.SpecularColor, 0);
	case PPI_SubsurfaceColor:
		return IsSubsurfaceModel(GBuffer.ShadingModelID) ? MaterialFloat4( ExtractSubsurfaceColor(GBuffer), GBuffer.CustomData.a ) : GBuffer.CustomData;
	case PPI_BaseColor:
		return MaterialFloat4(GBuffer.BaseColor, 0);
	case PPI_Specular:
		return GBuffer.Specular;
	case PPI_Metallic:
		return GBuffer.Metallic;
	case PPI_WorldNormal:
		return MaterialFloat4(GBuffer.WorldNormal, 0);
	case PPI_SeparateTranslucency:
		return MaterialFloat4(1, 1, 1, 1);	// todo
	case PPI_Opacity:
		return GBuffer.CustomData.a;
	case PPI_Roughness:
		return GBuffer.Roughness;
	case PPI_MaterialAO:
		return GBuffer.GBufferAO;
	case PPI_CustomDepth:
		return GBuffer.CustomDepth;
#if POST_PROCESS_MATERIAL
	case PPI_PostProcessInput0:
	{
		MaterialFloat4 Input0 = Texture2DSample(PostProcessInput_0_Texture, PostProcessInput_0_SharedSampler, UV);
		#if POST_PROCESS_MATERIAL_BEFORE_TONEMAP && METAL_ES3_1_PROFILE
		// Decode the input color since the color is encoded for MSAA 
		// The decode instructions might be able to skip with dynamic branch
		if (bMetalMSAAHDRDecode)
		{
			Input0.rgb = Input0.rgb * rcp(Input0.r*(-0.299) + Input0.g*(-0.587) + Input0.b*(-0.114) + 1.0);
		}
		#endif
		#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
		// We do a comparison of the POST_PROCESS_PROPAGATE_ALPHA_INPUT define with the PPI input index, to support propagation of alpha for shaders with UserSceneTextures,
		// where the specific PPI_PostProcessInput that has SceneColor as an input is dynamic.  Usually, POST_PROCESS_PROPAGATE_ALPHA_INPUT will be a constant, when a normal
		// SceneTexture node is used.
		if (POST_PROCESS_PROPAGATE_ALPHA_INPUT == PPI_PostProcessInput0)
		{
			PropagatedAlpha = Input0.a;
		}
		#endif
		return Input0;
	}

	#define RETURN_MOBILE_POST_PROCESS_INPUT_SAMPLE(Index) \
		return Texture2DSample(PostProcessInput_##Index##_Texture, PostProcessInput_##Index##_SharedSampler, UV)

	case PPI_PostProcessInput1:		RETURN_MOBILE_POST_PROCESS_INPUT_SAMPLE(1);
	case PPI_PostProcessInput2:		RETURN_MOBILE_POST_PROCESS_INPUT_SAMPLE(2);
	case PPI_PostProcessInput3:		RETURN_MOBILE_POST_PROCESS_INPUT_SAMPLE(3);
	case PPI_PostProcessInput4:		RETURN_MOBILE_POST_PROCESS_INPUT_SAMPLE(4);

#endif // POST_PROCESS_MATERIAL
	case PPI_DecalMask:
		return 0;  // material compiler will return an error
	case PPI_ShadingModelColor:
		return MaterialFloat4(GetShadingModelColor(GBuffer.ShadingModelID), 1);
	case PPI_ShadingModelID:
		return MaterialFloat4(GBuffer.ShadingModelID, 0, 0, 0);
	case PPI_AmbientOcclusion:
		return 1;
	case PPI_CustomStencil:
		return GBuffer.CustomStencil;
	case PPI_StoredBaseColor:
		return MaterialFloat4(GBuffer.StoredBaseColor, 0);
	case PPI_StoredSpecular:
		return MaterialFloat4(GBuffer.StoredSpecular.rrr, 0);
	case PPI_WorldTangent:
		return MaterialFloat4(GBuffer.WorldTangent, 0);
#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
	case PPI_PropagatedAlpha:
		return MaterialFloat4(PropagatedAlpha, 0, 0, 0);
#endif
	default:
		return MaterialFloat4(0, 0, 0, 0);
	}
#endif
}

#endif // SHADING_PATH_MOBILE

#if SHADING_PATH_DEFERRED

#if POST_PROCESS_MATERIAL
/** Samples the screen-space velocity for the specified UV coordinates in Buffer Space. */
float2 PostProcessVelocityLookup(float Depth, float2 UV)
{
	float4 EncodedVelocity = Texture2DSampleLevel(SceneTexturesStruct.GBufferVelocityTexture, SceneTexturesStruct_GBufferVelocityTextureSampler, UV, 0);

	float2 Velocity;
	if( EncodedVelocity.x > 0.0 )
	{
		Velocity = DecodeVelocityFromTexture(EncodedVelocity).xy;
	}
	else
	{
		float2 ViewportUV = BufferUVToViewportUV(UV);
		float2 ScreenPos = ViewportUVToScreenPos(ViewportUV);
		float4 ThisClip = float4( ScreenPos, Depth, 1 );
		float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
		Velocity = ScreenPos - PrevScreen;
	}

	return Velocity;
}
#endif

/** Applies an offset to the scene texture lookup and decodes the HDR linear space color. */
float4 SceneTextureLookup(float2 UV, int SceneTextureIndex, bool bFiltered)
{
#if SCENE_TEXTURES_DISABLED

	// When scene textures are disabled, the output is matched to the dummy scene texture defaults.
	switch(SceneTextureIndex)
	{
	case PPI_SceneDepth:
	case PPI_CustomDepth:
		return ConvertFromDeviceZ(0.0f);
	case PPI_MaterialAO:
	case PPI_CustomStencil:
		return 1.0f;
	default:
		return 0.0f;
	}

#elif MATERIAL_SHADINGMODEL_SINGLELAYERWATER && !HIT_PROXY_SHADER

	if (SceneTextureIndex == PPI_CustomDepth)
	{
		MaterialFloat Depth = ConvertFromDeviceZ(Texture2DSample(SingleLayerWater.CustomDepthTexture, SingleLayerWater.CustomDepthSampler, UV).r);
		return MaterialFloat4(Depth.rrr, 0.f);
	}
	else if (SceneTextureIndex == PPI_CustomStencil)
	{
		MaterialFloat Stencil = SingleLayerWater.CustomStencilTexture.Load(int3(UV * View.BufferSizeAndInvSize.xy, 0)) STENCIL_COMPONENT_SWIZZLE;
		return MaterialFloat4(Stencil, Stencil, Stencil, 0.f);
	}

#else // SCENE_TEXTURES_DISABLED

#if MATERIAL_DOMAIN_SURFACE &&  MATERIALBLENDING_ANY_TRANSLUCENT
	// BufferToSceneTextureScale is only needed for translucent surfaces today and we know that here.
	const uint2 PixelPos = uint2(UV * View.BufferSizeAndInvSize.xy * View.BufferToSceneTextureScale.xy);
#else
	const uint2 PixelPos = uint2(UV * View.BufferSizeAndInvSize.xy);
#endif

#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
	// Propagation of SceneColor alpha, to preserve it in materials that don't themselves write to alpha, is supported by storing the propagated alpha to a local variable,
	// which can be read from with a call to this function at the end of the shader.  The big advantage of this approach is that it even works with Custom HLSL, since it
	// just requires the SceneTextureLookup code to be reached by any means.  It also works well with UserSceneTexture fetches, where a given input can be dynamically set
	// to SceneColor.  Some complexity is added in that SceneTexture or UserSceneTexture nodes feeding into custom HLSL, where the input pin value itself isn't used, now
	// need to be explicitly dead stripped, as the side effect of writing to PropagatedAlpha prevents the shader compiler from optimizing out calls to SceneTextureLookup
	// based on the return value being unused in downstream code.  See UMaterialExpressionCustom::Compile for where that is handled.
	static float PropagatedAlpha = 0.0f;

	// There are three parts to the condition to detect whether we should automatically propagate SceneColor alpha:
	//		1.  To make the shader compiler optimize out repeated reads of alpha, we set a local bool to ensure we only read alpha from a given scene texture index once.
	//		2.  We want to support propagation of alpha for UserSceneTextures.  A given input is a UserSceneTexture if it's not used as a regular SceneTexture, as
	//			specified by the POST_PROCESS_USED_SCENE_TEXTURES mask compile define, so we skip propagation for any input with a bit set there.
	//			PPI_PostProcessInput0 is an exception, which is the SceneColor itself when used as a SceneTexture, and should always support propagation.
	//		3.  Finally, we test if the given input equals POST_PROCESS_PROPAGATE_ALPHA_INPUT, which will either be PPI_PostProcessInput0 if the material in question
	//			directly uses that as a SceneTexture, or the dynamic UserSceneTextureSceneColorInput variable, if UserSceneTexture nodes are in use, and one of them
	//			may be mapped to SceneColor.
	// If PostProcessInput0 is directly fetched in a SceneTexture node, the conditional will be compile time invariant, and no extra code is generated, making the alpha
	// propagation free.  Otherwise, for UserSceneTextures, only the final part of the comparison is dynamic, and the rest will be compile time invariant, so it adds one
	// scalar ALU (equals) and one vector ALU (conditional move), although the first UserSceneTexture vector ALU is free, rolled into the output assignment.  The material
	// can be set up to output opacity to optimize that minor overhead out completely if desired.
	//
	#define RETURN_POST_PROCESS_INPUT_SAMPLE(Index) {  \
			float4 Result = Texture2DSample(PostProcessInput_##Index##_Texture, bFiltered ? PostProcessInput_BilinearSampler : PostProcessInput_##Index##_SharedSampler, UV);  \
			static bool bFirstRead = true;  \
			if (bFirstRead && !(POST_PROCESS_USED_SCENE_TEXTURES & (1u << (Index + PPI_PostProcessInput0)) & ~(1u<<PPI_PostProcessInput0)) && POST_PROCESS_PROPAGATE_ALPHA_INPUT == PPI_PostProcessInput##Index)  \
			{ PropagatedAlpha = Result.a; }  \
			bFirstRead = false;  \
			return Result;  \
		}

#else
	#define RETURN_POST_PROCESS_INPUT_SAMPLE(Index) \
		return Texture2DSample(PostProcessInput_##Index##_Texture, bFiltered ? PostProcessInput_BilinearSampler : PostProcessInput_##Index##_SharedSampler, UV)
#endif

	// 1. Post-process inputs which are independent on material data
	switch (SceneTextureIndex)
	{
		case PPI_SceneColor:			return float4(CalcSceneColor(UV), 0);
		case PPI_SceneDepth:			return CalcSceneDepth(UV);
		case PPI_SeparateTranslucency:	return float4(1, 1, 1, 1);	// todo
		case PPI_CustomDepth:			return CalcSceneCustomDepth(UV);
	#if POST_PROCESS_MATERIAL
		case PPI_PostProcessInput0:		RETURN_POST_PROCESS_INPUT_SAMPLE(0);
		case PPI_PostProcessInput1:		RETURN_POST_PROCESS_INPUT_SAMPLE(1);
		case PPI_PostProcessInput2:		RETURN_POST_PROCESS_INPUT_SAMPLE(2);
		case PPI_PostProcessInput3:		RETURN_POST_PROCESS_INPUT_SAMPLE(3);
		case PPI_PostProcessInput4:		RETURN_POST_PROCESS_INPUT_SAMPLE(4);
	#endif // POST_PROCESS_MATERIAL
		case PPI_DecalMask:				return 0;  // material compiler will return an error
		case PPI_AmbientOcclusion:		return CalcSceneAO(UV);
		case PPI_CustomStencil:			return CalcSceneCustomStencil(PixelPos);
	#if POST_PROCESS_MATERIAL
		case PPI_Velocity:				return float4(PostProcessVelocityLookup(ConvertToDeviceZ(CalcSceneDepth(UV)), UV), 0, 0);
	#endif
	#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
		case PPI_PropagatedAlpha:		return float4(PropagatedAlpha, 0, 0, 0);
	#endif
	}

	// 2. Post-process inputs which are dependent on material data
	//    Note: For PostProcess material we always return the first layer dataCalcSceneCustomStencil
	#if SUBSTRATE_ENABLED && !SHADING_PATH_MOBILE && IS_DBUFFER_DECAL
	{
		// A special decal path to avoid requiring the binding of substrate data in the decal passes. Decals can only access depth, stencil and depth-from-normals or reprojected.
		// But not on mobile that is not using dbuffer decal but the legacy GBuffer blending.
		switch (SceneTextureIndex)
		{
		case PPI_DiffuseColor:			return 0.0f;
		case PPI_SpecularColor:			return 0.0f;
		case PPI_SubsurfaceColor:		return 0.0f;
		case PPI_BaseColor:				return 0.0f;
		case PPI_Specular:				return 0.0f;
		case PPI_Metallic:				return 0.0f;
		case PPI_WorldNormal:			return GetDBufferReprojectedWorldNormal(UV); // DBuffer uses depth buffer gradients or reprojection of last frame normal.
		case PPI_Opacity:				return 1.0f;
		case PPI_Roughness:				return 0.0f;
		case PPI_MaterialAO:			return 0.0f;
		case PPI_ShadingModelColor:		return 0.0f;
		case PPI_ShadingModelID:		return 0.0f;
		case PPI_StoredBaseColor:		return 0.0f;
		case PPI_StoredSpecular:		return 0.0f;
		case PPI_WorldTangent:			return 0.0f;
		case PPI_Anisotropy:			return 0.0f;
		case PPI_IsFirstPerson:			return 0.0f;
		}
	}
	#elif SUBSTRATE_ENABLED && (SUBSTRATE_GBUFFER_FORMAT==0 || SUBSTRATE_GBUFFER_FORMAT==1) && !SHADING_PATH_MOBILE // Not on mobile that is not using dbuffer decal but the legacy GBuffer blending.
	{
		#if !SUBSTRATE_OPAQUE_DEFERRED || ALLOW_SUBSTRATE_PUBLIC_ACCESS // Prevents opaque material to sample the Gbuffer.
		FSubstrateGBuffer SubstrateGBuffer = SubstratePublic_GetSubstrateGBufferFromFirstBSDF(PixelPos, false/*bGetNormalizedNormal*/);
		switch (SceneTextureIndex)
		{
			// order needs to match to ESceneTextureId
			case PPI_DiffuseColor:			return float4(SubstrateGBuffer.DiffuseColor, 0);
			case PPI_SpecularColor:			return float4(SubstrateGBuffer.SpecularColor, 0);
			case PPI_SubsurfaceColor:		return float4(SubstrateGBuffer.SubsurfaceColor, 0);
			case PPI_BaseColor:				return float4(SubstrateGBuffer.BaseColor, 0);
			case PPI_Specular:				return SubstrateGBuffer.Specular;
			case PPI_Metallic:				return SubstrateGBuffer.Metallic;
			case PPI_WorldNormal:			return float4(SubstrateGBuffer.WorldNormal, 0);
			case PPI_Opacity:				return SubstrateGBuffer.Opacity;
			case PPI_Roughness:				return SubstrateGBuffer.Roughness;
			case PPI_MaterialAO:			return SubstrateGBuffer.MaterialAO;
			case PPI_ShadingModelColor:		return float4(SubstrateGBuffer.ShadingModelColor, 1);
			case PPI_ShadingModelID:		return float4(SubstrateGBuffer.ShadingModelID, 0, 0, 0);
			case PPI_StoredBaseColor:		return float4(SubstrateGBuffer.StoredBaseColor, 0);
			case PPI_StoredSpecular:		return float4(SubstrateGBuffer.StoredSpecular.rrr, 0);
			case PPI_WorldTangent:			return float4(SubstrateGBuffer.WorldTangent, 0);
			case PPI_Anisotropy:			return SubstrateGBuffer.Anisotropy;
			case PPI_IsFirstPerson:			return SubstrateGBuffer.bIsFirstPerson ? 1.0f : 0.0f;
		}
		#endif
	}
	#else // SUBSTRATE_ENABLED
	{
		FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV, false/*bGetNormalizedNormal*/);
		switch (SceneTextureIndex)
		{
			// order needs to match to ESceneTextureId
			case PPI_DiffuseColor:			return float4(ScreenSpaceData.GBuffer.DiffuseColor, 0);
			case PPI_SpecularColor:			return float4(ScreenSpaceData.GBuffer.SpecularColor, 0);
			case PPI_SubsurfaceColor:		return IsSubsurfaceModel(ScreenSpaceData.GBuffer.ShadingModelID) ? float4(ExtractSubsurfaceColor(ScreenSpaceData.GBuffer), ScreenSpaceData.GBuffer.CustomData.a) : ScreenSpaceData.GBuffer.CustomData;
			case PPI_BaseColor:				return float4(ScreenSpaceData.GBuffer.BaseColor, 0);
			case PPI_Specular:				return ScreenSpaceData.GBuffer.Specular;
			case PPI_Metallic:				return ScreenSpaceData.GBuffer.Metallic;
			#if IS_DBUFFER_DECAL
			case PPI_WorldNormal:			return GetDBufferReprojectedWorldNormal(UV); // DBuffer uses depth buffer gradients or reprojection of last frame normal.
			#else
			case PPI_WorldNormal:			return float4(ScreenSpaceData.GBuffer.WorldNormal, 0);
			#endif
			case PPI_Opacity:				return ScreenSpaceData.GBuffer.CustomData.a;
			case PPI_Roughness:				return ScreenSpaceData.GBuffer.Roughness;
			case PPI_MaterialAO:			return ScreenSpaceData.GBuffer.GBufferAO;
			case PPI_ShadingModelColor:		return float4(GetShadingModelColor(ScreenSpaceData.GBuffer.ShadingModelID), 1);
			case PPI_ShadingModelID:		return float4(ScreenSpaceData.GBuffer.ShadingModelID, 0, 0, 0);
			case PPI_StoredBaseColor:		return float4(ScreenSpaceData.GBuffer.StoredBaseColor, 0);
			case PPI_StoredSpecular:		return float4(ScreenSpaceData.GBuffer.StoredSpecular.rrr, 0);
			case PPI_WorldTangent:			return float4(ScreenSpaceData.GBuffer.WorldTangent, 0);
			case PPI_Anisotropy:			return ScreenSpaceData.GBuffer.Anisotropy;
			case PPI_IsFirstPerson:			return IsFirstPerson(ScreenSpaceData.GBuffer) ? 1.0f : 0.0f;
		}
	}
	#endif // SUBSTRATE_ENABLED

#endif // SCENE_TEXTURES_DISABLED

	return float4(0, 0, 0, 0);
}

#endif // SHADING_PATH_DEFERRED

#define SceneTextureFetch(SceneTextureIndex, PixelOffset) SceneTextureFetchFunc(Parameters, SceneTextureIndex, PixelOffset)

/**
 * Utility function and macro to streamline Custom HLSL fetching from SceneTextures or UserSceneTextures with a pixel offset from default UVs, including
 * multiplication by texture size and clamping.  Example that fetches at offset 1,1 from the default sample:
 *
 *     SceneTextureFetch(PPI_PostProcessInput0, float2(1,1));
 */
float4 SceneTextureFetchFunc(FMaterialPixelParameters Parameters, int SceneTextureIndex, float2 PixelOffset)
{
#if SCENE_TEXTURES_DISABLED || !POST_PROCESS_MATERIAL
	return float4(0,0,0,0);
#else
	float2 UV = ClampSceneTextureUV(GetDefaultSceneTextureUV(Parameters, SceneTextureIndex) + GetSceneTextureBufferSize(SceneTextureIndex).zw * PixelOffset, SceneTextureIndex);
#if SHADING_PATH_MOBILE
	return MobileSceneTextureLookup(Parameters, SceneTextureIndex, UV);
#else
	return SceneTextureLookup(UV, SceneTextureIndex, true);
#endif
#endif
}

float4 SceneTextureFetchFunc(FMaterialPixelParameters Parameters, int SceneTextureIndex, float PixelOffsetX, float PixelOffsetY)
{
	return SceneTextureFetchFunc(Parameters, SceneTextureIndex, float2(PixelOffsetX, PixelOffsetY));
}

float4 SceneTextureLookupCommon(FMaterialPixelParameters Parameters, float2 UV, int SceneTextureIndex, bool bFiltered)
{
#if SHADING_PATH_MOBILE
	return MobileSceneTextureLookup(Parameters, SceneTextureIndex, UV);
#else
	return SceneTextureLookup(UV, SceneTextureIndex, bFiltered);
#endif
}

// SceneTextureLookup function used by new translator.  Shading path agnostic, and handles conditional application of exposure scale, so generated code doesn't need to vary.
float4 SceneTextureLookup(FMaterialPixelParameters Parameters, float2 UV, int SceneTextureIndex, bool bFiltered)
{
#if SHADING_PATH_MOBILE
	float4 Lookup = MobileSceneTextureLookup(Parameters, SceneTextureIndex, UV);
#else
	float4 Lookup = SceneTextureLookup(UV, SceneTextureIndex, bFiltered);
#endif
#if POST_PROCESS_MATERIAL && POST_PROCESS_MATERIAL_BEFORE_TONEMAP && !POST_PROCESS_DISABLE_PRE_EXPOSURE_SCALE
	if (SceneTextureIndex >= PPI_PostProcessInput0 && SceneTextureIndex <= PPI_PostProcessInput6)
	{
		Lookup.xyz *= View.OneOverPreExposure;
	}
#endif
	return Lookup;
}
#endif // NEEDS_SCENE_TEXTURES

#if SHADING_PATH_DEFERRED || MOBILE_DEFERRED_SHADING

/** Applies an offset to the scene texture lookup and decodes the HDR linear space color and alpha. */
float4 DecodeSceneColorAndAlpharForMaterialNode(float2 ScreenUV)
{
#if !defined(SceneColorCopyTexture)
	// Hit proxies rendering pass doesn't have access to valid render buffers
	return float4(0.0f, 0.0f, 0.0f, 0.0f);
#else
	float4 EncodedSceneColor = Texture2DSample(SceneColorCopyTexture, SceneColorCopySampler, ScreenUV);

	// Undo the function in EncodeSceneColorForMaterialNode
	float3 SampledColor = pow(EncodedSceneColor.rgb, 4) * 10;

	SampledColor *= View.OneOverPreExposure.xxx;

	return float4(SampledColor, EncodedSceneColor.a);
#endif
}

/** Applies an offset to the scene texture lookup and decodes the HDR linear space color. */
float3 DecodeSceneColorForMaterialNode(float2 ScreenUV)
{
	return DecodeSceneColorAndAlpharForMaterialNode(ScreenUV).rgb;
}

#endif // SHADING_PATH_DEFERRED || MOBILE_DEFERRED_SHADING

float4 MaterialExpressionDBufferTextureLookup(FMaterialPixelParameters Parameters, float2 BufferUV, int DBufferTextureIndex)
{
	if ((GetPrimitiveData(Parameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER) != 0 && View.ShowDecalsMask > 0)
	{
		uint2 PixelPos = uint2(BufferUV * View.BufferSizeAndInvSize.xy);
		uint ValidDBufferTargetMask = GetDBufferTargetMask(PixelPos) & (1u << DBufferTextureIndex);
#if SUBSTRATE_ENABLED
		const FSubstrateDBuffer DBufferData = SubstrateGetDBufferData(BufferUV, ValidDBufferTargetMask);
		switch (DBufferTextureIndex)
		{
		// All data (BaseColor, WorldNormal, Roughness) are pre-multiplied by their coverage
		case 0:	return float4(DBufferData.BaseColor,	DBufferData.OneMinusCoverage_BaseColor);
		case 1:	return float4(DBufferData.WorldNormal,	DBufferData.OneMinusCoverage_WorldNormal);
		case 2:	return float4(DBufferData.Roughness,	DBufferData.Metallic, DBufferData.Specular, DBufferData.OneMinusCoverage_Roughness);
		}
#else
		FDBufferData DBufferData = GetDBufferData(BufferUV, ValidDBufferTargetMask);
		switch (DBufferTextureIndex)
		{
		case 0:	return float4(DBufferData.PreMulColor, DBufferData.ColorOpacity);
		case 1:	return float4(DBufferData.PreMulWorldNormal, DBufferData.NormalOpacity);
		case 2:	return float4(DBufferData.PreMulRoughness, DBufferData.PreMulMetallic, DBufferData.PreMulSpecular, DBufferData.RoughnessOpacity);
		}
#endif
	}

	return float4(0, 0, 0, 1);
}


MaterialFloat2 GetDefaultPathTracingBufferTextureUV(FMaterialPixelParameters Parameters, const uint PathTracingBufferTextureIndex)
{
	float2 ViewportUV = GetViewportUV(Parameters);
#if POST_PROCESS_MATERIAL && SHADING_PATH_DEFERRED && MATERIAL_PATH_TRACING_BUFFER_READ && PATH_TRACING_POST_PROCESS_MATERIAL
	switch (PathTracingBufferTextureIndex)
	{
	case 0: return ViewportUV * PathTracingPostProcessInput_0_UVViewportSize + PathTracingPostProcessInput_0_UVViewportMin;
	case 1: return ViewportUV * PathTracingPostProcessInput_1_UVViewportSize + PathTracingPostProcessInput_1_UVViewportMin;
	case 2: return ViewportUV * PathTracingPostProcessInput_2_UVViewportSize + PathTracingPostProcessInput_2_UVViewportMin;
	case 3: return ViewportUV * PathTracingPostProcessInput_3_UVViewportSize + PathTracingPostProcessInput_3_UVViewportMin;
	case 4: return ViewportUV * PathTracingPostProcessInput_4_UVViewportSize + PathTracingPostProcessInput_4_UVViewportMin;
	}
#endif
	return ViewportUV;
}

float4 MaterialExpressionPathTracingBufferTextureLookup(FMaterialPixelParameters Parameters, float2 BufferUV, int PathTracingBufferTextureIndex)
{
#if POST_PROCESS_MATERIAL && SHADING_PATH_DEFERRED && MATERIAL_PATH_TRACING_BUFFER_READ && PATH_TRACING_POST_PROCESS_MATERIAL
	switch (PathTracingBufferTextureIndex)
	{
	case 0:	return Texture2DSample(PathTracingPostProcessInput_0_Texture, PathTracingPostProcessInput_0_Sampler, BufferUV);
	case 1:	return Texture2DSample(PathTracingPostProcessInput_1_Texture, PathTracingPostProcessInput_1_Sampler, BufferUV);
	case 2:	return Texture2DSample(PathTracingPostProcessInput_2_Texture, PathTracingPostProcessInput_2_Sampler, BufferUV);
	case 3:	return Texture2DSample(PathTracingPostProcessInput_3_Texture, PathTracingPostProcessInput_3_Sampler, BufferUV);
	case 4:	return Texture2DSample(PathTracingPostProcessInput_4_Texture, PathTracingPostProcessInput_4_Sampler, BufferUV);
	}
#endif
	return float4(1, 0, 0, 1);
}

float4 NeuralTextureOutput(FMaterialPixelParameters Parameters, float2 ViewportUV)
{
#if POST_PROCESS_MATERIAL && SHADING_PATH_DEFERRED && MATERIAL_NEURAL_POST_PROCESS
	
	float2 BufferUV = ViewportUV * PostProcessInput_0_UVViewportSize + PostProcessInput_0_UVViewportMin;
	float4 TextureValue = Texture2DSampleLevel(NeuralTexture, PostProcessInput_BilinearSampler, BufferUV, 0);
	return TextureValue;
#else
	return float4(0.0f, 0.0f, 0.0f, 1.0f);
#endif

}

float4 NeuralBufferOutput(FMaterialPixelParameters Parameters, float4 BufferIndices)
{
#if POST_PROCESS_MATERIAL && SHADING_PATH_DEFERRED && MATERIAL_NEURAL_POST_PROCESS

	float3 TextureValue = 0;
	
	int Batch = (int)BufferIndices.x;
	int StartChannel = (int)BufferIndices.y;
	float2 BufferWH = BufferIndices.zw;

	ReadFromNeuralBuffer(Batch, StartChannel, BufferWH, TextureValue);

	return float4(TextureValue, 1.0f);
#else
	return float4(0.0f, 0.0f, 0.0f, 1.0f);
#endif
}

// DERIV_BASE_VALUE() is a disgusting macro to manage backwards compatibility while changing the generated materials. The existing nodes are not "aware"
// of partial derivatives, and will use sprintf() to put the same line in both CalcPixelMaterialInputs() and CalcPixelMaterialInputsAnalyticDerivatives().
// If we have a line like this in CalcPixelMaterialInputs():
//     float Local0 = ...;
// It will look like this in CalcPixelMaterialInputsAnalyticDerivatives()
//     FloatDeriv Local0 = ...;
//
// That's ok, since the Local0 line is aware of derivatives. But if we have a line that isn't aware then we would want the CalcPixelMaterialInputs() version to be:
//     float Local1 = Local0 + 3.2;
// But the CalcPixelMaterialInputsAnalyticDerivatives() version would be:
//     float Local1 = Local0.Value + 3.2;
//
// It's not possible to emit two different versions (with and without ".Value") unless we change every single material node. So the workaround is to always emit
// DERIV_BASE_VALUE, and in one version #define it to ".Value" and in the other to "". That way, we can emit the same code to both functions. Ideally, once every
// single node is at least aware of derivatives (and can output two different versions) then we can remove this #define and #undef for DERIV_BASE_VALUE

#define DERIV_BASE_VALUE(_X) _X

#define SwizzleDeriv1(_V, _MASK) ConstructFloatDeriv( _V.Value._MASK, _V.Ddx._MASK, _V.Ddy._MASK)
#define SwizzleDeriv2(_V, _MASK) ConstructFloatDeriv2(_V.Value._MASK, _V.Ddx._MASK, _V.Ddy._MASK)
#define SwizzleDeriv3(_V, _MASK) ConstructFloatDeriv3(_V.Value._MASK, _V.Ddx._MASK, _V.Ddy._MASK)
#define SwizzleDeriv4(_V, _MASK) ConstructFloatDeriv4(_V.Value._MASK, _V.Ddx._MASK, _V.Ddy._MASK)

// Uniform material expressions.
SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

%{material_cache_common}

%{uniform_material_expressions}

%{custom_outputs}

%{custom_functions}

SHADER_POP_WARNINGS_STATE

#if USE_ANALYTIC_DERIVATIVES && TEXTURE_SAMPLE_DEBUG
MaterialFloat4 DebugTextureCommon(const int Mode, float2 UV, MaterialFloat2 DDX, MaterialFloat2 DDY, MaterialFloat Scale)
{
	const float DerivScale = View.GeneralPurposeTweak2 * 100.0f;
	if (Mode == 2)
	{
		return MaterialFloat4(UV, 0.0f, 0.0f);
	}
	else if (Mode == 3)
	{
		return MaterialFloat4(DDX * DerivScale + 0.5f, 0.0f, 0.0f);
	}
	else if (Mode == 4)
	{
		const float2 FiniteDDX = ddx(UV) * Scale;
		return MaterialFloat4(FiniteDDX * DerivScale + 0.5f, 0.0f, 0.0f);
	}
	else if (Mode == 5)
	{
		return MaterialFloat4(DDY * DerivScale + 0.5f, 0.0f, 0.0f);
	}
	else if(Mode == 6)
	{
		const float2 FiniteDDY = ddy(UV) * Scale;
		return MaterialFloat4(FiniteDDY * DerivScale + 0.5f, 0.0f, 0.0f);
	}
	else
	{
		return MaterialFloat4(0.0f, 0.0f, 0.0f, 0.0f);
	}
}

MaterialFloat4 DebugTexture2DSampleGrad(Texture2D Tex, SamplerState Sampler, float2 UV, MaterialFloat2 DDX, MaterialFloat2 DDY, MaterialFloat Scale)
{
	const int Mode = round(View.GeneralPurposeTweak);
	if (Mode > 1)
		return DebugTextureCommon(Mode, UV, DDX, DDY, Scale);
	else
		return Tex.SampleGrad(Sampler, UV, DDX, DDY);
}

#if NUM_VIRTUALTEXTURE_SAMPLES || LIGHTMAP_VT_ENABLED
MaterialFloat4 DebugTextureVirtualSample(
	Texture2D Physical, SamplerState PhysicalSampler,
	VTPageTableResult PageTableResult, uint LayerIndex,
	VTUniform Uniform, MaterialFloat Scale)
{
	const int Mode = round(View.GeneralPurposeTweak);
	if(Mode > 1)
		return DebugTextureCommon(Mode, PageTableResult.UV, PageTableResult.dUVdx, PageTableResult.dUVdy, Scale);
	else
		return TextureVirtualSample(Physical, PhysicalSampler, PageTableResult, LayerIndex, Uniform);
}
MaterialFloat4 DebugTextureVirtualSampleLevel(
	Texture2D Physical, SamplerState PhysicalSampler,
	VTPageTableResult PageTableResult, uint LayerIndex,
	VTUniform Uniform, MaterialFloat Scale)
{
	const int Mode = round(View.GeneralPurposeTweak);
	if (Mode > 1)
		return MaterialFloat4(0.0f, 0.0f, 0.0f, 0.0f);
	else
		return TextureVirtualSampleLevel(Physical, PhysicalSampler, PageTableResult, LayerIndex, Uniform);
}
#endif
#endif

// can return in tangent space or world space (use MATERIAL_TANGENTSPACENORMAL)
half3 GetMaterialNormalRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Normal;
}

half3 GetMaterialNormal(FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs)
{
	half3 RetNormal;

	RetNormal = GetMaterialNormalRaw(PixelMaterialInputs);
		
	#if (USE_EDITOR_SHADERS && !ES3_1_PROFILE) || MOBILE_EMULATION
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		half3 OverrideNormal = ResolvedView.NormalOverrideParameter.xyz;

		#if !MATERIAL_TANGENTSPACENORMAL
			OverrideNormal = Parameters.TangentToWorld[2] * (1 - ResolvedView.NormalOverrideParameter.w);
		#endif

		RetNormal = RetNormal * ResolvedView.NormalOverrideParameter.w + OverrideNormal;
	}
	#endif

	return RetNormal;
}

half3 GetMaterialTangentRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Tangent;
}

half3 GetMaterialTangent(FPixelMaterialInputs PixelMaterialInputs)
{
	return GetMaterialTangentRaw(PixelMaterialInputs);
}

half3 GetMaterialEmissiveRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.EmissiveColor;
}

half3 GetMaterialEmissive(FPixelMaterialInputs PixelMaterialInputs)
{
	half3 EmissiveColor = GetMaterialEmissiveRaw(PixelMaterialInputs);
#if !MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR
	EmissiveColor = max(EmissiveColor, 0.0f);
#endif
	return EmissiveColor;
}

half3 GetMaterialEmissiveForCS(FMaterialPixelParameters Parameters)
{
SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS
%{get_material_emissive_for_cs};
SHADER_POP_WARNINGS_STATE
}

// Shading Model is an uint and represents a SHADINGMODELID_* in ShadingCommon.ush 
uint GetMaterialShadingModel(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.ShadingModel;
}

half3 GetMaterialBaseColorRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.BaseColor;
}

half3 GetMaterialBaseColor(FPixelMaterialInputs PixelMaterialInputs)
{
	return saturate(GetMaterialBaseColorRaw(PixelMaterialInputs));
}

half GetMaterialMetallicRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Metallic;
}

half GetMaterialMetallic(FPixelMaterialInputs PixelMaterialInputs)
{
	return saturate(GetMaterialMetallicRaw(PixelMaterialInputs));
}

half GetMaterialSpecularRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Specular;
}

half GetMaterialSpecular(FPixelMaterialInputs PixelMaterialInputs)
{
	return saturate(GetMaterialSpecularRaw(PixelMaterialInputs));
}

half GetMaterialRoughnessRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Roughness;
}

half GetMaterialRoughness(FPixelMaterialInputs PixelMaterialInputs)
{
#if MATERIAL_FULLY_ROUGH
	return 1;
#endif
	half Roughness = saturate(GetMaterialRoughnessRaw(PixelMaterialInputs));
	
	#if (USE_EDITOR_SHADERS && !ES3_1_PROFILE) || MOBILE_EMULATION
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		Roughness = Roughness * ResolvedView.RoughnessOverrideParameter.y + ResolvedView.RoughnessOverrideParameter.x;
	}
	#endif
	
	return Roughness;
}

half GetMaterialAnisotropyRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Anisotropy;
}

half GetMaterialAnisotropy(FPixelMaterialInputs PixelMaterialInputs)
{
	return clamp(GetMaterialAnisotropyRaw(PixelMaterialInputs), -1.0f, 1.0f);
}

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

half GetMaterialTranslucencyDirectionalLightingIntensity()
{
%{get_material_translucency_directional_lighting_intensity};
}

half GetMaterialTranslucentShadowDensityScale()
{
%{get_material_translucent_shadow_density_scale};
}

half GetMaterialTranslucentSelfShadowDensityScale()
{
%{get_material_translucent_self_shadow_density_scale};
}

half GetMaterialTranslucentSelfShadowSecondDensityScale()
{
%{get_material_translucent_self_shadow_second_density_scale};
}

half GetMaterialTranslucentSelfShadowSecondOpacity()
{
%{get_material_translucent_self_shadow_second_opacity};
}

half GetMaterialTranslucentBackscatteringExponent()
{
%{get_material_translucent_backscattering_exponent};
}

half3 GetMaterialTranslucentMultipleScatteringExtinction()
{
%{get_material_translucent_multiple_scattering_extinction};
}

// This is the clip value constant that is defined in the material (range 0..1)
// Use GetMaterialMask() to get the Material Mask combined with this.
half GetMaterialOpacityMaskClipValue()
{
%{get_material_opacity_mask_clip_value};
}

SHADER_POP_WARNINGS_STATE


#if USES_DISPLACEMENT && IS_NANITE_PASS

float GetMaterialDisplacement(FPixelMaterialInputs PixelMaterialInputs)
{
	return saturate(PixelMaterialInputs.Displacement);
}

#endif

// Should only be used by GetMaterialOpacity(), returns the unmodified value generated from the shader expressions of the opacity input.
// To compute the opacity depending on the material blending GetMaterialOpacity() should be called instead.
half GetMaterialOpacityRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Opacity;
}

#if MATERIALBLENDING_MASKED
// Returns the material mask value generated from the material expressions.
// Use GetMaterialMask() to get the value altered depending on the material blend mode.
half GetMaterialMaskInputRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.OpacityMask;
}

// Returns the material mask value generated from the material expressions minus the used defined
// MaskClip value constant. If this value is <=0 the pixel should be killed.
half GetMaterialMask(FPixelMaterialInputs PixelMaterialInputs)
{
	return GetMaterialMaskInputRaw(PixelMaterialInputs) - GetMaterialOpacityMaskClipValue();
}
#endif

// Returns the material opacity depending on the material blend mode.
half GetMaterialOpacity(FPixelMaterialInputs PixelMaterialInputs)
{
	// Clamp to valid range to prevent negative colors from lerping
	return saturate(GetMaterialOpacityRaw(PixelMaterialInputs));
}

#if TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY
half GetMaterialMaskedOpacity(FPixelMaterialInputs PixelMaterialInputs)
{
	return GetMaterialOpacity(PixelMaterialInputs) - GetMaterialOpacityMaskClipValue();
}
#endif

bool IsFirstPerson_FromFlags(uint Flags)
{
#if SUPPORT_FIRST_PERSON_RENDERING
	return (Flags & PRIMITIVE_SCENE_DATA_FLAG_IS_FIRST_PERSON) != 0;
#else
	return false;
#endif
}

bool IsFirstPerson(FMaterialVertexParameters Parameters)
{
	return IsFirstPerson_FromFlags(GetPrimitiveData(Parameters).Flags);
}

bool IsFirstPerson(FMaterialPixelParameters Parameters)
{
	return IsFirstPerson_FromFlags(GetPrimitiveData(Parameters).Flags);
}

float3 TransformToFirstPerson(float3 TranslatedWorldPosition, float FirstPersonInterpolationAlpha)
{
#if SUPPORT_FIRST_PERSON_RENDERING
	const float3 FirstPersonPosition = mul(TranslatedWorldPosition, (float3x3)ResolvedView.FirstPersonTransform);
	const float3 InterpolatedPosition = lerp(TranslatedWorldPosition, FirstPersonPosition, FirstPersonInterpolationAlpha);
	return InterpolatedPosition;
#else
	return TranslatedWorldPosition;
#endif
}

float3 TransformToPreviousFirstPerson(float3 TranslatedWorldPosition, float FirstPersonInterpolationAlpha)
{
#if SUPPORT_FIRST_PERSON_RENDERING
	const float3 FirstPersonPosition = mul(TranslatedWorldPosition, (float3x3)ResolvedView.PrevFirstPersonTransform);
	const float3 InterpolatedPosition = lerp(TranslatedWorldPosition, FirstPersonPosition, FirstPersonInterpolationAlpha);
	return InterpolatedPosition;
#else
	return TranslatedWorldPosition;
#endif
}

float GetMaterialFirstPersonInterpolationAlpha(FMaterialVertexParameters Parameters)
{
#if SUPPORT_FIRST_PERSON_RENDERING && HAVE_GetFirstPersonOutput0
	return saturate(GetFirstPersonOutput0(Parameters));
#else
	return 1.0f;
#endif
}

float GetMaterialPreviousFirstPersonInterpolationAlpha(FMaterialVertexParameters Parameters)
{
#if SUPPORT_FIRST_PERSON_RENDERING && HAVE_GetFirstPersonOutput0Prev
	return saturate(GetFirstPersonOutput0Prev(Parameters));
#else
	return 1.0f;
#endif
}

void ApplyMaterialFirstPersonTransform(FMaterialVertexParameters Parameters, inout float3 WorldPositionWithWPO)
{
#if SUPPORT_FIRST_PERSON_RENDERING
	BRANCH
	if (IsFirstPerson(Parameters))
	{
		const float LerpAlpha = GetMaterialFirstPersonInterpolationAlpha(Parameters);
		const float3 FirstPersonPosition = TransformToFirstPerson(WorldPositionWithWPO, LerpAlpha);
		WorldPositionWithWPO = FirstPersonPosition;
	}
#endif
}

void ApplyMaterialPreviousFirstPersonTransform(FMaterialVertexParameters Parameters, inout float3 PreviousWorldPositionWithWPO)
{
#if SUPPORT_FIRST_PERSON_RENDERING
	BRANCH
	if (IsFirstPerson(Parameters))
	{
		const float LerpAlpha = GetMaterialPreviousFirstPersonInterpolationAlpha(Parameters);
		const float3 PreviousFirstPersonPosition = TransformToPreviousFirstPerson(PreviousWorldPositionWithWPO, LerpAlpha);
		PreviousWorldPositionWithWPO = PreviousFirstPersonPosition;
	}
#endif
}

bool ShouldEnableWorldPositionOffset(FMaterialVertexParameters Parameters)
{
#if USES_WORLD_POSITION_OFFSET
	if (!Parameters.bEvaluateWorldPositionOffset ||
		(GetPrimitiveData(Parameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET) == 0)
	{
		return false;
	}
	return true;
#else
	return false;
#endif
}

float3 ClampWorldPositionOffset(FMaterialVertexParameters Parameters, float3 InOffset)
{
#if (FEATURE_LEVEL == FEATURE_LEVEL_ES3_1 && VF_SUPPORTS_PRIMITIVE_SCENE_DATA)
	// Do not use WPO clamping on mobile platforms as MaxWPOExtent has to be pulled from PrimitiveUB breaking auto-instancing
	return InOffset;
#else
	const float MaxWPODim = GetPrimitiveData(Parameters).MaxWPOExtent;
	return MaxWPODim <= 0.0f ? InOffset : clamp(InOffset, -MaxWPODim.xxx, MaxWPODim.xxx);
#endif
}

float3 GetMaterialWorldPositionOffsetRaw(FMaterialVertexParameters Parameters)
{
#if ENABLE_NEW_HLSL_GENERATOR
	return Parameters.WorldPositionOffset;
#else
	SHADER_PUSH_WARNINGS_STATE
	SHADER_DISABLE_WARNINGS
	%{get_material_world_position_offset_raw};
	SHADER_POP_WARNINGS_STATE
#endif
}

float3 GetMaterialWorldPositionOffset(FMaterialVertexParameters Parameters)
{
	BRANCH
	if (ShouldEnableWorldPositionOffset(Parameters))
	{
		return ClampWorldPositionOffset(Parameters, GetMaterialWorldPositionOffsetRaw(Parameters));
	}
	return float3(0, 0, 0);
}

float3 GetMaterialPreviousWorldPositionOffsetRaw(FMaterialVertexParameters Parameters)
{
#if ENABLE_NEW_HLSL_GENERATOR
	return Parameters.WorldPositionOffset;
#else
	SHADER_PUSH_WARNINGS_STATE
	SHADER_DISABLE_WARNINGS
	%{get_material_previous_world_position_offset_raw};
	SHADER_POP_WARNINGS_STATE
#endif
}

float3 GetMaterialPreviousWorldPositionOffset(FMaterialVertexParameters Parameters)
{
	BRANCH
	if (ShouldEnableWorldPositionOffset(Parameters))
	{
		return ClampWorldPositionOffset(Parameters, GetMaterialPreviousWorldPositionOffsetRaw(Parameters));
	}
	return float3(0, 0, 0);
}

float GetMaterialTemporalResponsiveness(FMaterialPixelParameters Parameters)
{
	#if HAVE_GetTemporalResponsivenessOutput0 > 0
		return saturate(GetTemporalResponsivenessOutput0(Parameters));
	#else
		return 0;
	#endif
}

float3 GetMaterialMotionVectorWorldOffset(FMaterialPixelParameters Parameters)
{
	#if HAVE_GetMotionVectorWorldOffsetOutput0 > 0
		return GetMotionVectorWorldOffsetOutput0(Parameters);
	#else
		return float3(0, 0, 0);
	#endif
}

// .rgb:SubsurfaceColor, .a:SSProfileId in 0..1 range
MaterialFloat4 GetMaterialSubsurfaceDataRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.Subsurface;
}

MaterialFloat4 GetMaterialSubsurfaceData(FPixelMaterialInputs PixelMaterialInputs)
{
	MaterialFloat4 OutSubsurface = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs);
	OutSubsurface.rgb = saturate(OutSubsurface.rgb);
	return OutSubsurface;
}

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

half GetMaterialCustomData0(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.CustomData0;
}

half GetMaterialCustomData1(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.CustomData1;
}

SHADER_POP_WARNINGS_STATE

half GetMaterialAmbientOcclusionRaw(FPixelMaterialInputs PixelMaterialInputs)
{
	half AmbientOcclusion = PixelMaterialInputs.AmbientOcclusion;

	#if (USE_EDITOR_SHADERS && !ES3_1_PROFILE) || MOBILE_EMULATION
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		AmbientOcclusion = AmbientOcclusion * ResolvedView.AmbientOcclusionOverrideParameter.y + ResolvedView.AmbientOcclusionOverrideParameter.x;
	}
	#endif

	return AmbientOcclusion;
}

half GetMaterialAmbientOcclusion(FPixelMaterialInputs PixelMaterialInputs)
{
	return saturate(GetMaterialAmbientOcclusionRaw(PixelMaterialInputs));
}

struct FMaterialRefractionData
{
	float2 Data;
	float  RefractionDepthBias;
};

float GetUnlitMaterialRefractionIOR(in FMaterialRefractionData RefractionData)
{
	return RefractionData.Data.x;
}

float GetMaterialRefractionIOR(in FMaterialRefractionData RefractionData)
{
#if REFRACTION_USE_INDEX_OF_REFRACTION
	return RefractionData.Data.x;
#else
	return 1.0f;
#endif
}

float GetMaterialRefractionPixelNormalStrength(in FMaterialRefractionData RefractionData)
{
#if REFRACTION_USE_PIXEL_NORMAL_OFFSET
	return RefractionData.Data.x;
#else
	return 1.0f;
#endif
}

float2 GetMaterialRefraction2DOffset(in FMaterialRefractionData RefractionData)
{
#if REFRACTION_USE_2D_OFFSET
	return RefractionData.Data.xy;
#else
	return float2(0.0f, 0.0f);
#endif
}

FMaterialRefractionData GetMaterialRefraction(FPixelMaterialInputs PixelMaterialInputs)
{
	FMaterialRefractionData Data;
	Data.Data					= PixelMaterialInputs.Refraction.xy;
	Data.RefractionDepthBias	= PixelMaterialInputs.Refraction.z;
	return Data;
}

#if NUM_TEX_COORD_INTERPOLATORS

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

void GetMaterialCustomizedUVs(FMaterialVertexParameters Parameters, inout float2 OutTexCoords[NUM_TEX_COORD_INTERPOLATORS])
{
	// Always initialise to avoid later operations to potentially cause compiler failure due to read uninitialized value when get_material_customized_u_vs is not defined.
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		OutTexCoords[CoordinateIndex] = 0.0f;
	}

%{get_material_customized_u_vs}
}

void GetCustomInterpolators(FMaterialVertexParameters Parameters, inout float2 OutTexCoords[NUM_TEX_COORD_INTERPOLATORS])
{
%{get_custom_interpolators}
}

SHADER_POP_WARNINGS_STATE

#endif

float GetMaterialPixelDepthOffset(FPixelMaterialInputs PixelMaterialInputs)
{
	return PixelMaterialInputs.PixelDepthOffset;
}

#if DECAL_PRIMITIVE

float3 TransformTangentNormalToWorld(MaterialFloat3x3 TangentToWorld, float3 TangentNormal)
{
	// To transform the normals use tranpose(Inverse(DecalToWorld)) = transpose(WorldToDecal)
	// But we want to only rotate the normals (we don't want to non-uniformaly scale them).
	// We assume the matrix is only a scale and rotation, and we remove non-uniform scale:
				
	// Pre-multiply by the inverse of the non-uniform scale in DecalToWorld
	float4 ScaledNormal = float4(-TangentNormal.z * DecalToWorldInvScale.x, TangentNormal.y * DecalToWorldInvScale.y, TangentNormal.x * DecalToWorldInvScale.z, 0.f);

	// Compute the normal 
	return normalize(mul(ScaledNormal, DecalToWorld).xyz);
}

#else //DECAL_PRIMITIVE

float3 TransformTangentNormalToWorld(MaterialFloat3x3 TangentToWorld, float3 TangentNormal)
{
	return normalize(float3(TransformTangentVectorToWorld(TangentToWorld, TangentNormal)));
}

#endif //DECAL_PRIMITIVE

float3 CalculateAnisotropyTangent(in out FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs)
{
	float3 Normal = Parameters.WorldNormal;

#if CLEAR_COAT_BOTTOM_NORMAL && (NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0)
	Normal =  ClearCoatBottomNormal0(Parameters);
	#if MATERIAL_TANGENTSPACENORMAL
		Normal = TransformTangentVectorToWorld(Parameters.TangentToWorld, Normal);
	#endif
#endif

	float3 Tangent = GetMaterialTangent(PixelMaterialInputs);

#if MATERIAL_TANGENTSPACENORMAL
	Tangent = TransformTangentNormalToWorld(Parameters.TangentToWorld, Tangent);
#endif

	float3 BiTangent = cross(Normal, Tangent);
	Tangent = normalize(cross(BiTangent, Normal));

	return Tangent;
}

float NormalCurvatureToRoughness(float3 WorldNormal, float3 dNdx, float3 dNdy)
{
	float x = dot(dNdx, dNdx);
	float y = dot(dNdy, dNdy);
	float CurvatureApprox = pow(max(x, y), View.NormalCurvatureToRoughnessScaleBias.z);
	return saturate(CurvatureApprox * View.NormalCurvatureToRoughnessScaleBias.x + View.NormalCurvatureToRoughnessScaleBias.y);
}

float NormalCurvatureToRoughness(float3 WorldNormal)
{
	return NormalCurvatureToRoughness(WorldNormal, ddx(WorldNormal), ddy(WorldNormal));
}

float GetRoughnessFromNormalCurvature(FMaterialPixelParameters InMaterialParameters)
{
#if MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS
	// Curvature-to-roughness uses derivatives of the WorldVertexNormal, which is incompatible with centroid interpolation because
	// the samples are not uniformly distributed. Therefore we use WorldVertexNormal_Center which is guaranteed to be center interpolated.
	#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
		#if USE_ANALYTIC_DERIVATIVES
		return NormalCurvatureToRoughness(InMaterialParameters.WorldVertexNormal_Center, InMaterialParameters.WorldGeoNormal_DDX, InMaterialParameters.WorldGeoNormal_DDY);
		#else
		return NormalCurvatureToRoughness(InMaterialParameters.WorldVertexNormal_Center);
		#endif
	#else
		#if USE_ANALYTIC_DERIVATIVES
		return NormalCurvatureToRoughness(InMaterialParameters.TangentToWorld[2].xyz, InMaterialParameters.WorldGeoNormal_DDX, InMaterialParameters.WorldGeoNormal_DDY);
		#else
		return NormalCurvatureToRoughness(InMaterialParameters.TangentToWorld[2].xyz);
		#endif
	#endif
#else
	return 0.f;
#endif
}

#if HAIR_STRAND_MESH_FACTORY
float3 RotateLocalHairTangent(float3 In);
#endif

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

void EvaluateVertexMaterialAttributes(inout FMaterialVertexParameters Parameters)
{
	%{evaluate_vertex_material_attributes}
}

void EvaluatePrevVertexMaterialAttributes(inout FMaterialVertexParameters Parameters)
{
	%{evaluate_vertex_material_attributes_prev}
}

SHADER_POP_WARNINGS_STATE

// We must to flip the normal of two sided SLW material with normal specified in world space.
#define TWO_SIDED_WORLD_SPACE_SINGLELAYERWATER_NORMAL (MATERIAL_SHADINGMODEL_SINGLELAYERWATER && !MATERIAL_TANGENTSPACENORMAL && MATERIAL_TWOSIDED)

// Be very, very careful about changing CalcPixelMaterialInputs() or CalcPixelMaterialInputsAnalyticDerivatives(). They apply the same basic calculation,
// but CalcPixelMaterialInputs() uses finite difference derivates from hardware whereas CalcPixelMaterialInputsAnalyticDerivatives() uses analytic
// derivatives. If you change anything, make sure that both functions stay identical. 
void CalcPixelMaterialInputs(in out FMaterialPixelParameters Parameters, in out FPixelMaterialInputs PixelMaterialInputs)
{
SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS
	// Initial calculations (required for Normal)
%{calc_pixel_material_inputs_initial_calculations}
	// The Normal is a special case as it might have its own expressions and also be used to calculate other inputs, so perform the assignment here
%{calc_pixel_material_inputs_normal}
SHADER_POP_WARNINGS_STATE

#if TEMPLATE_USES_SUBSTRATE
	Parameters.SubstratePixelFootprint = SubstrateGetPixelFootprint(Parameters.WorldPosition_CamRelative, GetRoughnessFromNormalCurvature(Parameters));
	Parameters.SharedLocalBases = SubstrateInitialiseSharedLocalBases();
	Parameters.SubstrateTree = GetInitialisedSubstrateTree();
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1
	Parameters.SharedLocalBasesFullySimplified = SubstrateInitialiseSharedLocalBases();
	Parameters.SubstrateTreeFullySimplified = GetInitialisedSubstrateTree();
#endif
#endif

	// Note that here MaterialNormal can be in world space or tangent space
	float3 MaterialNormal = GetMaterialNormal(Parameters, PixelMaterialInputs);

#if MATERIAL_TANGENTSPACENORMAL

#if RAYHITGROUPSHADER && HAIR_STRAND_MESH_FACTORY && HAIRSTRANDS_HAS_NORMAL_CONNECTED
	MaterialNormal = RotateLocalHairTangent(MaterialNormal);
#endif

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	// Mobile will rely on only the final normalize for performance
	MaterialNormal = normalize(MaterialNormal);
#endif

	// normalizing after the tangent space to world space conversion improves quality with sheared bases (UV layout to WS causes shrearing)
	// use full precision normalize to avoid overflows
	Parameters.WorldNormal = TransformTangentNormalToWorld(Parameters.TangentToWorld, MaterialNormal);

#else //MATERIAL_TANGENTSPACENORMAL

	Parameters.WorldNormal = normalize(MaterialNormal);

#endif //MATERIAL_TANGENTSPACENORMAL

// Hair strands/cards ignore TwoSided as WorldNormal is interpreted as a tangent, and thus is not affected by TwoSided
#if (MATERIAL_TANGENTSPACENORMAL || TWO_SIDED_WORLD_SPACE_SINGLELAYERWATER_NORMAL)  && !(HAIR_STRAND_MESH_FACTORY || HAIR_CARD_MESH_FACTORY)
	// flip the normal for backfaces being rendered with a two-sided material
	Parameters.WorldNormal *= Parameters.TwoSidedSign;
#endif

	Parameters.ReflectionVector = ReflectionAboutCustomWorldNormal(Parameters, Parameters.WorldNormal, false);

#if !PARTICLE_SPRITE_FACTORY
	Parameters.Particle.MotionBlurFade = 1.0f;
#endif // !PARTICLE_SPRITE_FACTORY

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS
	// Now the rest of the inputs
%{calc_pixel_material_inputs_other_inputs}
SHADER_POP_WARNINGS_STATE

#if MATERIAL_USES_ANISOTROPY
	Parameters.WorldTangent = CalculateAnisotropyTangent(Parameters, PixelMaterialInputs);
#else
	Parameters.WorldTangent = 0;
#endif
}
#undef DERIV_BASE_VALUE

#if USE_ANALYTIC_DERIVATIVES

#define DERIV_BASE_VALUE(_X) _X.Value
void CalcPixelMaterialInputsAnalyticDerivatives(in out FMaterialPixelParameters Parameters, in out FPixelMaterialInputs PixelMaterialInputs)
{
SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS
	// Initial calculations (required for Normal)
%{calc_pixel_material_inputs_analytic_derivatives_initial}
	// The Normal is a special case as it might have its own expressions and also be used to calculate other inputs, so perform the assignment here
%{calc_pixel_material_inputs_analytic_derivatives_normal}
SHADER_POP_WARNINGS_STATE

#if TEMPLATE_USES_SUBSTRATE
	Parameters.SubstratePixelFootprint = SubstrateGetPixelFootprint(Parameters.WorldPosition_CamRelative, GetRoughnessFromNormalCurvature(Parameters));
	Parameters.SharedLocalBases = SubstrateInitialiseSharedLocalBases();
	Parameters.SubstrateTree = GetInitialisedSubstrateTree();
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1
	Parameters.SharedLocalBasesFullySimplified = SubstrateInitialiseSharedLocalBases();
	Parameters.SubstrateTreeFullySimplified = GetInitialisedSubstrateTree();
#endif
#endif

	// Note that here MaterialNormal can be in world space or tangent space
	float3 MaterialNormal = GetMaterialNormal(Parameters, PixelMaterialInputs);

#if MATERIAL_TANGENTSPACENORMAL

#if RAYHITGROUPSHADER && HAIR_STRAND_MESH_FACTORY && HAIRSTRANDS_HAS_NORMAL_CONNECTED
	MaterialNormal = RotateLocalHairTangent(MaterialNormal);
#endif

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	// Mobile will rely on only the final normalize for performance
	MaterialNormal = normalize(MaterialNormal);
#endif

	// normalizing after the tangent space to world space conversion improves quality with sheared bases (UV layout to WS causes shrearing)
	// use full precision normalize to avoid overflows
	Parameters.WorldNormal = TransformTangentNormalToWorld(Parameters.TangentToWorld, MaterialNormal);

#else //MATERIAL_TANGENTSPACENORMAL

	Parameters.WorldNormal = normalize(MaterialNormal);

#endif //MATERIAL_TANGENTSPACENORMAL

// Hair strands/cards ignore TwoSided as WorldNormal is interpreted as a tangent, and thus is not affected by TwoSided
#if (MATERIAL_TANGENTSPACENORMAL || TWO_SIDED_WORLD_SPACE_SINGLELAYERWATER_NORMAL) && !(HAIR_STRAND_MESH_FACTORY || HAIR_CARD_MESH_FACTORY)
	// flip the normal for backfaces being rendered with a two-sided material
	Parameters.WorldNormal *= Parameters.TwoSidedSign;
#endif

	Parameters.ReflectionVector = ReflectionAboutCustomWorldNormal(Parameters, Parameters.WorldNormal, false);

#if !PARTICLE_SPRITE_FACTORY
	Parameters.Particle.MotionBlurFade = 1.0f;
#endif // !PARTICLE_SPRITE_FACTORY

SHADER_PUSH_WARNINGS_STATE
SHADER_DISABLE_WARNINGS

	// Now the rest of the inputs
%{calc_pixel_material_inputs_analytic_derivatives_other_inputs}

SHADER_POP_WARNINGS_STATE

#if MATERIAL_USES_ANISOTROPY
	Parameters.WorldTangent = CalculateAnisotropyTangent(Parameters, PixelMaterialInputs);
#else
	Parameters.WorldTangent = 0;
#endif
}
#undef DERIV_BASE_VALUE
#endif

// Programmatically set the line number after all the material inputs which have a variable number of line endings
// This allows shader error line numbers after this point to be the same regardless of which material is being compiled
#line %{line_number}

void ClipLODTransition(float2 SvPosition, float DitherFactor)
{
	if (abs(DitherFactor) > .001)
	{
		float ArgCos = dot(floor(SvPosition.xy), float2(347.83451793, 3343.28371963));
#if PLATFORM_SUPPORTS_RELAXED_PRECISION
		// Temporary workaround for precision issues on mobile when the argument is bigger than 10k
		ArgCos = fmod(ArgCos, 10000);
#endif
		float RandCos = cos(ArgCos);
		float RandomVal = frac(RandCos * 1000.0);
		half RetVal = (DitherFactor < 0.0) ?
			(DitherFactor + 1.0 > RandomVal) :
			(DitherFactor < RandomVal);
		clip(RetVal - .001);
	}
}

void ClipLODTransition(FMaterialPixelParameters Parameters, float DitherFactor)
{
	ClipLODTransition(Parameters.SvPosition.xy, DitherFactor);
}


#define REQUIRES_VF_ATTRIBUTES_FOR_CLIPPING ((USE_INSTANCING || USE_INSTANCE_CULLING) && USE_DITHERED_LOD_TRANSITION)

#if (USE_INSTANCING  || USE_INSTANCE_CULLING) && USE_DITHERED_LOD_TRANSITION
void ClipLODTransition(FMaterialPixelParameters Parameters)
{
	float LODFactor = select(PrimitiveDither.LODFactor != 0.0f, PrimitiveDither.LODFactor, Parameters.PerInstanceParams.z);
	if (LODFactor != 0.0)
	{
		ClipLODTransition(Parameters, LODFactor);
	}
}
#elif USE_DITHERED_LOD_TRANSITION && !USE_STENCIL_LOD_DITHER
void ClipLODTransition(FMaterialPixelParameters Parameters)
{
	if (PrimitiveDither.LODFactor != 0.0)
	{
		ClipLODTransition(Parameters, PrimitiveDither.LODFactor);
	}
}
void ClipLODTransition(float2 SvPosition)
{
	if (PrimitiveDither.LODFactor != 0.0)
	{
		ClipLODTransition(SvPosition, PrimitiveDither.LODFactor);
	}
}
#else
void ClipLODTransition(FMaterialPixelParameters Parameters)
{
}
void ClipLODTransition(float2 SvPosition)
{
}
#endif

void GetMaterialClippingShadowDepth(FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs)
{
	ClipLODTransition(Parameters);
	#if MATERIALBLENDING_MASKED
		clip(GetMaterialMask(PixelMaterialInputs));
	#elif TRANSLUCENT_SHADOW_WITH_MASKED_OPACITY
		clip(GetMaterialMaskedOpacity(PixelMaterialInputs));
	#elif MATERIALBLENDING_TRANSLUCENT
		clip(GetMaterialOpacity(PixelMaterialInputs) - 1.0f / 255.0f);
	#endif
}

#if MATERIAL_DITHER_OPACITY_MASK
float DitheredOpacityMaskToOpacity(float Mask)
{
	// This represents the expected value of the function GetMaterialCoverageAndClipping
	// which randomly dithers the fragment on or off to produce the effect of opacity

	// The expected value of this dithering can be computed as:
	//    Simplify[Integrate[If[Mask + Dither - 1/2 < 0, 0, 1], {Dither, 0, 1}]]
	// which is just:

	return saturate(Mask + 0.5);
}
#endif

#if MATERIALBLENDING_MASKED
float GetDitheredMaterialMask(FPixelMaterialInputs PixelMaterialInputs, uint2 SvPosition, uint FrameIndex)
{
	#if MATERIAL_DITHER_OPACITY_MASK
		return GetDitheredCoverage(SvPosition.xy, FrameIndex, GetMaterialMask(PixelMaterialInputs));
	#else
		return GetMaterialMask(PixelMaterialInputs);
	#endif
}
#endif

void GetMaterialCoverageAndClipping(FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs)
{
	ClipLODTransition(Parameters);

#if MATERIALBLENDING_MASKED && !SINGLE_LAYER_WATER_NO_DISCARD
	clip(GetDitheredMaterialMask(PixelMaterialInputs, Parameters.SvPosition.xy, View.StateFrameIndex));
#endif
}

// The material blending mode alone is not enough information. We also need to know if the material is ThinTranslucent because in this case we should never clip.
// Indeed, ThinTranslucent surfaces always have a coverage of 1 and opacity represent the coverage of the opaque lit material sitting on top of the translucent one.
void GetMaterialClippingVelocity(FMaterialPixelParameters Parameters, FPixelMaterialInputs PixelMaterialInputs, bool bIsThinTranslucent, float MaterialOpacity)
{
#if MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE || SUBSTRATE_TRANSLUCENT_MATERIAL
	ClipLODTransition(Parameters);
	clip(bIsThinTranslucent ? 1.0f : MaterialOpacity - 1.0 / 255.0 - GetMaterialOpacityMaskClipValue());
#else
	GetMaterialCoverageAndClipping(Parameters, PixelMaterialInputs);
#endif
}

#define MATERIALBLENDING_MASKED_USING_COVERAGE (FORWARD_SHADING && MATERIALBLENDING_MASKED && SUPPORTS_PIXEL_COVERAGE && MATERIAL_USE_ALPHA_TO_COVERAGE)
#if MATERIALBLENDING_MASKED_USING_COVERAGE

uint GetDerivativeCoverageFromMask(float MaterialMask)
{
	uint Coverage = 0x0;						// MSAA sampler mask support up to 8X
	if (MaterialMask > 0.010) Coverage = 0x08;	// 0000 1000
	if (MaterialMask > 0.125) Coverage = 0x18;	// 0001 1000
	if (MaterialMask > 0.250) Coverage = 0x19;	// 0001 1001
	if (MaterialMask > 0.375) Coverage = 0x39;	// 0011 1001
	if (MaterialMask > 0.500) Coverage = 0x3D;	// 0011 1101
	if (MaterialMask > 0.625) Coverage = 0x7D;	// 0111 1101
	if (MaterialMask > 0.750) Coverage = 0x7F;	// 0111 1111
	if (MaterialMask > 0.875) Coverage = 0xFF;	// 1111 1111
	return Coverage;
}

// Returns the new pixel coverage according the material's mask and the current pixel's mask.
uint DiscardMaterialWithPixelCoverage(FMaterialPixelParameters MaterialParameters, FPixelMaterialInputs PixelMaterialInputs)
{
	ClipLODTransition(MaterialParameters);
	float OriginalMask = GetMaterialMaskInputRaw(PixelMaterialInputs);
	float MaskClip = GetMaterialOpacityMaskClipValue();

	if (ResolvedView.NumSceneColorMSAASamples > 1)
	{
		float Mask = (OriginalMask - MaskClip) / (1.0 - MaskClip);
		uint CurrentPixelCoverage = GetDerivativeCoverageFromMask(Mask);
		// Discard pixel shader if all sample are masked to avoid computing other material inputs.
		clip(float(CurrentPixelCoverage) - 0.5);
		return CurrentPixelCoverage;
	}
	clip(OriginalMask - MaskClip);
	return 0xF;
}

#endif // MATERIALBLENDING_MASKED_USING_COVERAGE


#define FrontFaceSemantic SV_IsFrontFace
#define FIsFrontFace bool
half GetFloatFacingSign(FIsFrontFace bIsFrontFace)
{
#if COMPILER_DXC && (COMPILER_VULKAN || COMPILER_GLSL_ES3_1) && (!RAYHITGROUPSHADER) && (!IS_NANITE_PASS)
		// We need to flip SV_IsFrontFace for Vulkan when compiling with DXC due to different coordinate systems.
		// HLSLcc did that by flipping SV_IsFrontFace in the high-level GLSL output.
		// NOTE: Raytracing and Nanite don't need an SV_IsFrontFace workaround.
		return bIsFrontFace ? -1 : +1;
#else
		return bIsFrontFace ? +1 : -1;
#endif
}

#define OPTIONAL_IsFrontFace , in FIsFrontFace bIsFrontFace : FrontFaceSemantic

#ifndef VF_SUPPORTS_WORLD_POSITION_SHADER_OFFSETS
#define VF_SUPPORTS_WORLD_POSITION_SHADER_OFFSETS 1
#endif 
// Helper macro to globally ignore requests for non-offset world positions in materials when lower than shader model 4. We do this
// because we are using an extra interpolator for this second world position, and in < ES31 there may not be enough
// NOTE: Forced on in development shaders because the Out-Of-Bounds Pixels visualizer uses it
#define USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS	((NEEDS_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS || USE_DEVELOPMENT_SHADERS) && VF_SUPPORTS_WORLD_POSITION_SHADER_OFFSETS)

/** Initializes the subset of Parameters that was not set in GetMaterialPixelParameters. */
void CalcMaterialParametersEx(
	in out FMaterialPixelParameters Parameters,
	in out FPixelMaterialInputs PixelMaterialInputs,
	float4 SvPosition,
	float4 ScreenPosition,
	FIsFrontFace bIsFrontFace,
	float3 TranslatedWorldPosition,
	float3 TranslatedWorldPositionExcludingShaderOffsets)
{
	Parameters.WorldPosition_CamRelative = TranslatedWorldPosition;
	Parameters.WorldPosition_NoOffsets_CamRelative = TranslatedWorldPositionExcludingShaderOffsets;

	Parameters.LWCData = MakeMaterialLWCData(Parameters);

#if WSVECTOR_IS_TILEOFFSET
	Parameters.AbsoluteWorldPosition = DFMultiplyLHSAndFastTwoSum(Parameters.LWCData.AbsoluteWorldPosition.Tile, UE_LWC_RENDER_TILE_SIZE, Parameters.LWCData.AbsoluteWorldPosition.Offset);
#elif WSVECTOR_IS_DOUBLEFLOAT
	Parameters.AbsoluteWorldPosition = Parameters.LWCData.AbsoluteWorldPosition;
#endif

	// If the material uses any non-offset world position expressions, calculate those parameters. If not, 
	// the variables will have been initialised to 0 earlier.
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
#if WSVECTOR_IS_TILEOFFSET
	Parameters.WorldPosition_NoOffsets = DFMultiplyLHSAndFastTwoSum(Parameters.LWCData.WorldPosition_NoOffsets.Tile, UE_LWC_RENDER_TILE_SIZE, Parameters.LWCData.WorldPosition_NoOffsets.Offset);
#elif WSVECTOR_IS_DOUBLEFLOAT
	Parameters.WorldPosition_NoOffsets = Parameters.LWCData.WorldPosition_NoOffsets;
#endif
#endif

	Parameters.SvPosition = SvPosition;
	Parameters.ScreenPosition = ScreenPosition;
	Parameters.ViewBufferUV = ScreenPositionToBufferUV(ScreenPosition);

	// CameraVector is a normalised vector representing the "from surface to camera" direction.
	#if RAYHITGROUPSHADER
		Parameters.CameraVector = -WorldRayDirection();
	#else
		Parameters.CameraVector = select(IsOrthoProjection(ResolvedView), -ResolvedView.ViewForward, normalize(-Parameters.WorldPosition_CamRelative.xyz));
	#endif

	Parameters.LightVector = 0;

#if IS_NANITE_PASS && !IS_NANITE_TRANSLUCENCY
	// Nanite does not support OPTIONAL_IsFrontFace, so Nanite sets the following to 1.0f or -1.0f in
	// GetMaterialPixelParameters - here we pull it out into our own front facing bool.
	const bool bNaniteIsFrontFace = Parameters.TwoSidedSign > 0.0f;
#endif

	Parameters.TwoSidedSign = 1.0f;

#if MATERIAL_TWOSIDED && HAS_PRIMITIVE_UNIFORM_BUFFER
	// #dxr: DirectX Raytracing's HitKind() intrinsic already accounts for negative scaling
	#if !RAYHITGROUPSHADER
		Parameters.TwoSidedSign *= ResolvedView.CullingSign * GetPrimitive_DeterminantSign(Parameters.PrimitiveId);
	#endif
#endif

#if MATERIAL_TWOSIDED || RAYHITGROUPSHADER
	// Either we have a two-sided material that needs a sign flip, or we have a ray tracing material
	// that needs to consider rays arriving from either side
	#if IS_NANITE_PASS && !IS_NANITE_TRANSLUCENCY
		Parameters.TwoSidedSign *= GetFloatFacingSign(bNaniteIsFrontFace);
	#else
		Parameters.TwoSidedSign *= GetFloatFacingSign(bIsFrontFace);
	#endif
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED
	InitializeVirtualTextureFeedback(Parameters.VirtualTextureFeedback);
#endif

#if USE_ANALYTIC_DERIVATIVES
	if(!TEXTURE_SAMPLE_DEBUG || View.GeneralPurposeTweak >= 1.0f)
		CalcPixelMaterialInputsAnalyticDerivatives(Parameters, PixelMaterialInputs);
	else
#endif
	{
		CalcPixelMaterialInputs(Parameters, PixelMaterialInputs);
	}
}

// convenience function to setup CalcMaterialParameters assuming we don't support TranslatedWorldPositionExcludingShaderOffsets
// @param SvPosition from SV_Position when rendering the view, for other projections e.g. shadowmaps this function cannot be used and you need to call CalcMaterialParametersEx()
void CalcMaterialParameters(
	in out FMaterialPixelParameters Parameters,
	in out FPixelMaterialInputs PixelMaterialInputs,
	float4 SvPosition,
	FIsFrontFace bIsFrontFace)
{
	float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);

	CalcMaterialParametersEx(Parameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
}

// Postprocess output viewport may not match main view viewport.
// This function translates the 'true' SvPosition in the post pass to the equivalent SvPosition in the main view (pre upscaling, viewrect offset, ...).
float4 PostSvPositionToViewSvPosition(float2 ViewportUV, float4 PostPassSvPosition)
{
	float2 PixelPos = (ViewportUV * View.ViewSizeAndInvSize.xy) + View.ViewRectMin.xy;
	return float4(floor(PixelPos)+0.5, PostPassSvPosition.zw);
}

void CalcMaterialParametersPost(
	in out FMaterialPixelParameters Parameters,
	in out FPixelMaterialInputs PixelMaterialInputs,
	float4 PostPassSvPosition,
	float2 ViewportUV,
	FIsFrontFace bIsFrontFace)
{
	float4 ViewSvPosition = PostSvPositionToViewSvPosition(ViewportUV, PostPassSvPosition);
	float4 ScreenPosition = SvPositionToScreenPosition(ViewSvPosition);
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(ViewSvPosition);
	CalcMaterialParametersEx(Parameters, PixelMaterialInputs, PostPassSvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
}

/** Assemble the transform from tangent space into world space */
half3x3 AssembleTangentToWorld( half3 TangentToWorld0, half4 TangentToWorld2 )
{
	// Will not be orthonormal after interpolation. This perfectly matches xNormal.
	// Any mismatch with xNormal will cause distortions for baked normal maps.

	// Derive the third basis vector off of the other two.
	// Flip based on the determinant sign
	half3 TangentToWorld1 = cross(TangentToWorld2.xyz,TangentToWorld0) * TangentToWorld2.w;
	// Transform from tangent space to world space
	return half3x3(TangentToWorld0, TangentToWorld1, TangentToWorld2.xyz);
}

// Whether the material shader should output pixel depth offset
#define OUTPUT_PIXEL_DEPTH_OFFSET (WANT_PIXEL_DEPTH_OFFSET && !IS_VIRTUAL_TEXTURE_MATERIAL && !IS_NANITE_PASS && ((MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) || (TRANSLUCENT_WRITING_VELOCITY)))

// Whether to use the hidden d3d11 feature that supports depth writes with ZCull by only pushing into the screen
//@todo - use for other SM5 platforms
#define SUPPORTS_CONSERVATIVE_DEPTH_WRITES ((COMPILER_HLSL && FEATURE_LEVEL >= FEATURE_LEVEL_SM5) || COMPILER_PSSL || (COMPILER_METAL && FEATURE_LEVEL >= FEATURE_LEVEL_SM5) || SWITCH_PROFILE || SWITCH_PROFILE_FORWARD)
#define USE_CONSERVATIVE_DEPTH_WRITES (OUTPUT_PIXEL_DEPTH_OFFSET && SUPPORTS_CONSERVATIVE_DEPTH_WRITES) 

#if USE_CONSERVATIVE_DEPTH_WRITES

#if COMPILER_HLSL || COMPILER_METAL_SHADER_CONVERTER
	// Note: for some reason using SV_DepthLessEqual without these interpolation modifiers causes a compile error in d3d
	#define INPUT_POSITION_QUALIFIERS linear noperspective centroid
	// Use conservative depth output so we still get Z Cull.  Note, this is a reversed Z depth surface.
	#define DEPTH_WRITE_SEMANTIC SV_DepthLessEqual
#elif COMPILER_METAL
	#define INPUT_POSITION_QUALIFIERS 
	#define DEPTH_WRITE_SEMANTIC SV_DepthLessEqual
#elif COMPILER_PSSL
	#define INPUT_POSITION_QUALIFIERS
	#define DEPTH_WRITE_SEMANTIC S_DEPTH_LE_OUTPUT
#elif SWITCH_PROFILE || SWITCH_PROFILE_FORWARD
	#define INPUT_POSITION_QUALIFIERS
	#define DEPTH_WRITE_SEMANTIC SV_DepthLessEqual
#else
	#error USE_CONSERVATIVE_DEPTH_WRITES enabled for unsupported platform
#endif

#else
	#define INPUT_POSITION_QUALIFIERS 
	#define DEPTH_WRITE_SEMANTIC SV_DEPTH
#endif

#if OUTPUT_PIXEL_DEPTH_OFFSET
	#define OPTIONAL_OutDepthConservative ,out float OutDepth : DEPTH_WRITE_SEMANTIC
	#define OPTIONAL_OutDepth ,out float OutDepth : SV_DEPTH
#else
	#define OPTIONAL_OutDepthConservative
	#define OPTIONAL_OutDepth
#endif

// Must match EPixelDepthOffsetMode from EngineTypes.h
#define PODM_LEGACY			0
#define PODM_CAMERAVECTOR	1

float ApplyPixelDepthOffsetToMaterialParameters(inout FMaterialPixelParameters MaterialParameters, FPixelMaterialInputs PixelMaterialInputs, out float OutDepth)
{
	float PixelDepthOffset = GetMaterialPixelDepthOffset(PixelMaterialInputs);

	// SvPosition.z contains device depth value normally written to depth buffer
	// ScreenPosition.z is 'SvPosition.z * SvPosition.w'
	// So here we compute a new device depth value with the given pixel depth offset, but clamp the value against the regular SvPosition.z
	// This clamp is important, even if PixelDepthOffset is 0.0f, the computed DeviceDepth may end up 'slightly' larger than SvPosition.z due to floating point whatever
	// Since we are outputing depth with SV_DepthLessEqual, this ends up as undefined behavior
	// In particular, this can cause problems on PS4....PS4 enables RE_Z when using depth output along with virtual texture UAV feedback buffer writes
	// RE_Z causes the HW to perform depth test twice, once before executing pixel shader, and once after
	// The PreZ pass will write depth buffer using depth offset, then the base pass will test against this value using both modified and unmodifed depth
	// If the unmodified depth is ever slightly less than the modified depth, the initial depth test will fail, which results in z-fighting/flickering type artifacts
	float DeviceDepth;
	if(IsOrthoProjection(ResolvedView))
	{
		//Ortho projections have PDO applied to z rather than w. w should always stay as 1.0f
		float OriginalDeviceZ = MaterialParameters.SvPosition.z;
		float SceneDepth = ConvertFromDeviceZ(OriginalDeviceZ);
		DeviceDepth = max(SceneDepth + PixelDepthOffset, SceneDepth);

		PixelDepthOffset = SceneDepth - DeviceDepth;

		MaterialParameters.ScreenPosition.z = ConvertToDeviceZ(SceneDepth + PixelDepthOffset);
		MaterialParameters.SvPosition.z = MaterialParameters.ScreenPosition.z;

		DeviceDepth = ConvertToDeviceZ(DeviceDepth);
		PixelDepthOffset = MaterialParameters.SvPosition.z - OriginalDeviceZ;

		// In this Ortho case, CameraVector is always ViewForward.
		MaterialParameters.LWCData.AbsoluteWorldPosition = WSAdd(MaterialParameters.LWCData.AbsoluteWorldPosition, -MaterialParameters.CameraVector * PixelDepthOffset);
	}
	else
	{
		// We clamp PixelDepthOffset to 0 in order to not move the pixel towards the camera.
		const float3 CameraPixelForwardVector = -MaterialParameters.CameraVector;

	#if PIXEL_DEPTH_OFFSET_MODE == PODM_LEGACY

		// The legacy behavior applying a world space offset independently from the perspective projection.
		const float3 AbsoluteWorldOffsetVector = CameraPixelForwardVector * max(0.0, PixelDepthOffset);

	#elif PIXEL_DEPTH_OFFSET_MODE == PODM_CAMERAVECTOR

		// The change of depth will be less at the edge of the screen due to the perspective.
		const float PixelDepthOffetForPerspective = PixelDepthOffset * dot(CameraPixelForwardVector, ResolvedView.ViewForward);
		// This is the world space vector to offset the pixel world space position by according to the perspsective.
		// It clamps the offset to 0 in order to only push the pixel foward and match the depth clamping math below.
		const float3 AbsoluteWorldOffsetVector = CameraPixelForwardVector * max(0.0, PixelDepthOffetForPerspective);
		// Update the new pixel depth offset accounting for perspective projection.
		PixelDepthOffset = PixelDepthOffetForPerspective;

	#endif

		// Compute the new DeviceDepth accounting for the PixelDepthOffset.
		DeviceDepth = min(MaterialParameters.ScreenPosition.z / (MaterialParameters.ScreenPosition.w + PixelDepthOffset), MaterialParameters.SvPosition.z);

		// Once we've computed our (clamped) device depth, recompute PixelDepthOffset again to take the potential clamp into account
		PixelDepthOffset = (MaterialParameters.ScreenPosition.z - DeviceDepth * MaterialParameters.ScreenPosition.w) / DeviceDepth;

		// Update positions used for shading
		MaterialParameters.ScreenPosition.w += PixelDepthOffset;
		MaterialParameters.SvPosition.w = MaterialParameters.ScreenPosition.w;

		MaterialParameters.LWCData.AbsoluteWorldPosition = WSAdd(MaterialParameters.LWCData.AbsoluteWorldPosition, AbsoluteWorldOffsetVector);
	}

	MaterialParameters.AbsoluteWorldPosition = WSToDF(MaterialParameters.LWCData.AbsoluteWorldPosition);

	OutDepth = INVARIANT(DeviceDepth);

	return PixelDepthOffset;
}

float3 GetWorldBentNormalZero(in FMaterialPixelParameters MaterialParameters)
{
#if NUM_MATERIAL_OUTPUTS_GETBENTNORMAL > 0
	#if MATERIAL_TANGENTSPACENORMAL
		return normalize(TransformTangentVectorToWorld(MaterialParameters.TangentToWorld, GetBentNormal0(MaterialParameters)));
	#else
		return GetBentNormal0(MaterialParameters);
	#endif
#else
	return MaterialParameters.WorldNormal;
#endif
}

// Return the Iris & Iris plane normal, used for eye shading model
void GetEyeNormals(
	float IrisMask,
	float IrisDistance,
	in float3 InNormal,
	in float3 InClearCoatNormal,
	in float3 InCustomTangent,
	inout float3 OutIrisNormal,
	inout float3 OutIrisPlaneNormal)
{
#if IRIS_NORMAL
  #if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0 || SUBSTRATE_LEGACY_IRIS_NORMAL > 0
	OutIrisNormal = normalize(InClearCoatNormal);
  #else
	OutIrisNormal = InNormal;
  #endif

  #if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0 || SUBSTRATE_LEGACY_IRIS_TANGENT > 0
	OutIrisPlaneNormal = normalize(InCustomTangent);
  #else
	OutIrisPlaneNormal = OutIrisNormal;
  #endif
#else
  #if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
	OutIrisNormal = normalize(InCustomTangent);	// Weird that the tangent ends up in the normal, but that was the legacy behavior.
	OutIrisPlaneNormal = OutIrisNormal;
  #else
	OutIrisNormal = InNormal;
	OutIrisPlaneNormal = InNormal;
  #endif
#endif
}

#if MATERIAL_CACHE
%{material_cache_pass}
#endif // MATERIAL_CACHE
