// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define SAMPLERTYPE_Color 0
#define SAMPLERTYPE_Grayscale 1
#define SAMPLERTYPE_Alpha 2
#define SAMPLERTYPE_Normal 3
#define SAMPLERTYPE_Masks 4
#define SAMPLERTYPE_DistanceFieldFont 5
#define SAMPLERTYPE_LinearColor 6
#define SAMPLERTYPE_LinearGrayscale 7
#define SAMPLERTYPE_Data 8
#define SAMPLERTYPE_External 9

#define SAMPLERTYPE_VirtualColor 10
#define SAMPLERTYPE_VirtualGrayscale 11
#define SAMPLERTYPE_VirtualAlpha 12
#define SAMPLERTYPE_VirtualNormal 13
#define SAMPLERTYPE_VirtualMasks 14
#define SAMPLERTYPE_VirtualLinearColor 15
#define SAMPLERTYPE_VirtualLinearGrayscale 16

/**
 * Combined Texture/SamplerState types used by new HLSL translator
 * These currently need to be gated with the ENABLE_NEW_HLSL_GENERATOR define,
 * as HLSLcc can't handle structs with objects inside.
 * Before new HLSL translator can be used everywhere, we'll need to switch all platforms to DXC
 * Otherwise, this code will need to be changed
 */
#if ENABLE_NEW_HLSL_GENERATOR
struct FTexture2D
{
	Texture2D Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

struct FTexture2DArray
{
	Texture2DArray Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

struct FTextureCube
{
	TextureCube Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

struct FTextureCubeArray
{
	TextureCubeArray Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

struct FTexture3D
{
	Texture3D Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

struct FTextureExternal
{
	TextureExternal Texture;
	SamplerState Sampler;
	int SamplerType; // EMaterialSamplerType
};

FTexture2D MakeTexture2D(Texture2D Texture, SamplerState Sampler, int SamplerType)
{
	FTexture2D Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

FTexture2DArray MakeTexture2DArray(Texture2DArray Texture, SamplerState Sampler, int SamplerType)
{
	FTexture2DArray Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

FTextureCube MakeTexture2DArray(TextureCube Texture, SamplerState Sampler, int SamplerType)
{
	FTextureCube Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

FTextureCubeArray MakeTexture2DArray(TextureCubeArray Texture, SamplerState Sampler, int SamplerType)
{
	FTextureCubeArray Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

FTexture3D MakeTexture3D(Texture3D Texture, SamplerState Sampler, int SamplerType)
{
	FTexture3D Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

FTextureExternal MakeTextureExternal(TextureExternal Texture, SamplerState Sampler, int SamplerType)
{
	FTextureExternal Result;
	Result.Texture = Texture;
	Result.Sampler = Sampler;
	Result.SamplerType = SamplerType;
	return Result;
}

#if NUM_VIRTUALTEXTURE_SAMPLES
struct FTextureVirtual
{
	Texture2D<uint4> PageTable0;
	Texture2D<uint4> PageTable1;
	Texture2D<uint> PageTableIndirection;
	Texture2D PhysicalTexture;
	SamplerState PhysicalSampler;
	VTPageTableUniform PageTableUniform;
	VTUniform Uniform;
	int SamplerType; // EMaterialSamplerType
};
#endif // NUM_VIRTUALTEXTURE_SAMPLES

#endif // ENABLE_NEW_HLSL_GENERATOR

MaterialFloat4 ProcessMaterialColorTextureLookup(MaterialFloat4 TextureValue)
{
	return TextureValue;
}

MaterialFloat4 ProcessMaterialVirtualColorTextureLookup(MaterialFloat4 TextureValue)
{
	return ProcessMaterialColorTextureLookup(TextureValue);
}

MaterialFloat4 ProcessMaterialExternalTextureLookup(MaterialFloat4 TextureValue)
{
#if COMPILER_GLSL_ES3_1
	return MaterialFloat4(pow(TextureValue.rgb, 2.2f), TextureValue.a);
#else
	return ProcessMaterialColorTextureLookup(TextureValue);
#endif
}

MaterialFloat4 ProcessMaterialLinearColorTextureLookup(MaterialFloat4 TextureValue)
{
	return TextureValue;
}

MaterialFloat4 ProcessMaterialGreyscaleTextureLookup(MaterialFloat4 TextureValue)
{
	// Sampling a greyscale texture in D3D9 gives: (G,G,G)
	// Sampling a greyscale texture in D3D11 gives: (G,0,0)
	// This replication reproduces the D3D9 behavior in all cases.
	MaterialFloat GrayValue = TextureValue.r;
#if (COMPILER_GLSL_ES3_1 || VULKAN_PROFILE) // OpenGLES3.1, Vulkan3.1 do not support sRGB sampling from R8
	#if MOBILE_EMULATION
	if( ResolvedView.MobilePreviewMode > 0.5f )
	{
		// undo HW srgb->lin
		GrayValue = pow(GrayValue, 1.0f/2.2f); // TODO: replace with a more accurate lin -> sRGB conversion.
	}
	#endif
	// sRGB read approximation (in highp if possible)
	float LinValue = GrayValue;
	LinValue *= LinValue;
	return MaterialFloat4(LinValue.rrrr);
#endif 
	return GrayValue.rrrr;
}

MaterialFloat4 ProcessMaterialLinearGreyscaleTextureLookup(MaterialFloat4 TextureValue)
{
	// Sampling a greyscale texture in D3D9 gives: (G,G,G)
	// Sampling a greyscale texture in D3D11 gives: (G,0,0)
	// This replication reproduces the D3D9 behavior in all cases.
	return TextureValue.rrrr;
}

MaterialFloat4 ProcessMaterialAlphaTextureLookup(MaterialFloat4 TextureValue)
{
	// Sampling a single channel texture in D3D9 gives: (G,G,G)
	// Sampling a single channel texture in D3D11 gives: (G,0,0)
	// This replication reproduces the D3D9 behavior in all cases.
	return TextureValue.rrrr;
}

float4 ApplyMaterialSamplerType(float4 Value, int SamplerType)
{
	switch(SamplerType)
	{
	case SAMPLERTYPE_External: return ProcessMaterialExternalTextureLookup(Value);
	case SAMPLERTYPE_Color: return ProcessMaterialColorTextureLookup(Value);
	case SAMPLERTYPE_VirtualColor: return ProcessMaterialVirtualColorTextureLookup(Value);

	case SAMPLERTYPE_LinearColor:
	case SAMPLERTYPE_VirtualLinearColor: return ProcessMaterialLinearColorTextureLookup(Value);

	case SAMPLERTYPE_Alpha:
	case SAMPLERTYPE_VirtualAlpha:
	case SAMPLERTYPE_DistanceFieldFont: return ProcessMaterialAlphaTextureLookup(Value);

	case SAMPLERTYPE_Grayscale:
	case SAMPLERTYPE_VirtualGrayscale: return ProcessMaterialGreyscaleTextureLookup(Value);

	case SAMPLERTYPE_LinearGrayscale:
	case SAMPLERTYPE_VirtualLinearGrayscale: return ProcessMaterialLinearGreyscaleTextureLookup(Value);

	case SAMPLERTYPE_Normal:		return UnpackNormalMap(Value);
	case SAMPLERTYPE_VirtualNormal: return UnpackNormalMapVT(Value);

	case SAMPLERTYPE_Masks:
	case SAMPLERTYPE_VirtualMasks:
	case SAMPLERTYPE_Data:
	default: return Value;
	}
}
