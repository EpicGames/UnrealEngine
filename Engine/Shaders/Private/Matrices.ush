// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Creates an ortho projection matrix
//
// Similar to Core\Public\Math\OrthoMatrix.h
float4x4 ReversedZOrthoMatrix(float Width, float Height, float ZScale, float ZOffset)
{
	float4x4 O;
	O[0] = float4((Width) ? (1.0f / Width) : 1.0f, 0.0f, 0.0f, 0.0f);
	O[1] = float4(0.0f, (Height) ? (1.0f / Height) : 1.f, 0.0f, 0.0f);
	O[2] = float4(0.0f, 0.0f, -ZScale, 0.0f);
	O[3] = float4(0.0f, 0.0f, 1.0 - ZOffset * ZScale, 1.0f);
	return O;	
}

// Creates an perspective projection matrix
//
// Similar to Core\Public\Math\PerspectiveMatrix.h
float4x4 ReversedZPerspectiveMatrix(float HalfFOV, float Width, float Height, float MinZ, float MaxZ)
{
	float4x4 O;
	O[0] = float4(1.0f / tan(HalfFOV), 0.0f, 0.0f, 0.0f);
	O[1] = float4(0.0f, Width / tan(HalfFOV) / Height, 0.0f, 0.0f);
	O[2] = float4(0.0f, 0.0f, ((MinZ == MaxZ) ? 0.0f : MinZ / (MinZ - MaxZ)), 1.0f);
	O[3] = float4(0.0f, 0.0f, ((MinZ == MaxZ) ? MinZ : -MaxZ * MinZ / (MinZ - MaxZ)), 0.0f);
	return O;
}

// Creates a view matrix given an eye position, a position to look at, and an up vector. 
// This does the same thing as D3DXMatrixLookAtLH.
//
// Similar to Core\Public\Math\Matrix.inl
float4x4 LookAtMatrix(const float3 EyePosition, const float3 LookAtPosition, const float3 UpVector)
{
	float4x4 O;

	const float3 ZAxis = normalize(LookAtPosition - EyePosition);
	const float3 XAxis = normalize(cross(UpVector, ZAxis));
	const float3 YAxis = cross(ZAxis, XAxis);

	for (uint RowIndex = 0; RowIndex < 3; RowIndex++)
	{
		O[RowIndex][0] = XAxis[RowIndex];
		O[RowIndex][1] = YAxis[RowIndex];
		O[RowIndex][2] = ZAxis[RowIndex];
		O[RowIndex][3] = 0.0f;
	}

	O[3][0] = dot(-EyePosition, XAxis);
	O[3][1] = dot(-EyePosition, YAxis);
	O[3][2] = dot(-EyePosition, ZAxis);
	O[3][3] = 1.0f;

	return O;
}

float4x4 Inverse(float4x4 m)
{
	float n11 = m[0][0], n12 = m[1][0], n13 = m[2][0], n14 = m[3][0];
	float n21 = m[0][1], n22 = m[1][1], n23 = m[2][1], n24 = m[3][1];
	float n31 = m[0][2], n32 = m[1][2], n33 = m[2][2], n34 = m[3][2];
	float n41 = m[0][3], n42 = m[1][3], n43 = m[2][3], n44 = m[3][3];

	float t11 = n23 * n34 * n42 - n24 * n33 * n42 + n24 * n32 * n43 - n22 * n34 * n43 - n23 * n32 * n44 + n22 * n33 * n44;
	float t12 = n14 * n33 * n42 - n13 * n34 * n42 - n14 * n32 * n43 + n12 * n34 * n43 + n13 * n32 * n44 - n12 * n33 * n44;
	float t13 = n13 * n24 * n42 - n14 * n23 * n42 + n14 * n22 * n43 - n12 * n24 * n43 - n13 * n22 * n44 + n12 * n23 * n44;
	float t14 = n14 * n23 * n32 - n13 * n24 * n32 - n14 * n22 * n33 + n12 * n24 * n33 + n13 * n22 * n34 - n12 * n23 * n34;

	float det = n11 * t11 + n21 * t12 + n31 * t13 + n41 * t14;
	float idet = 1.0f / det;

	float4x4 ret;

	ret[0][0] = t11 * idet;
	ret[0][1] = (n24 * n33 * n41 - n23 * n34 * n41 - n24 * n31 * n43 + n21 * n34 * n43 + n23 * n31 * n44 - n21 * n33 * n44) * idet;
	ret[0][2] = (n22 * n34 * n41 - n24 * n32 * n41 + n24 * n31 * n42 - n21 * n34 * n42 - n22 * n31 * n44 + n21 * n32 * n44) * idet;
	ret[0][3] = (n23 * n32 * n41 - n22 * n33 * n41 - n23 * n31 * n42 + n21 * n33 * n42 + n22 * n31 * n43 - n21 * n32 * n43) * idet;

	ret[1][0] = t12 * idet;
	ret[1][1] = (n13 * n34 * n41 - n14 * n33 * n41 + n14 * n31 * n43 - n11 * n34 * n43 - n13 * n31 * n44 + n11 * n33 * n44) * idet;
	ret[1][2] = (n14 * n32 * n41 - n12 * n34 * n41 - n14 * n31 * n42 + n11 * n34 * n42 + n12 * n31 * n44 - n11 * n32 * n44) * idet;
	ret[1][3] = (n12 * n33 * n41 - n13 * n32 * n41 + n13 * n31 * n42 - n11 * n33 * n42 - n12 * n31 * n43 + n11 * n32 * n43) * idet;

	ret[2][0] = t13 * idet;
	ret[2][1] = (n14 * n23 * n41 - n13 * n24 * n41 - n14 * n21 * n43 + n11 * n24 * n43 + n13 * n21 * n44 - n11 * n23 * n44) * idet;
	ret[2][2] = (n12 * n24 * n41 - n14 * n22 * n41 + n14 * n21 * n42 - n11 * n24 * n42 - n12 * n21 * n44 + n11 * n22 * n44) * idet;
	ret[2][3] = (n13 * n22 * n41 - n12 * n23 * n41 - n13 * n21 * n42 + n11 * n23 * n42 + n12 * n21 * n43 - n11 * n22 * n43) * idet;

	ret[3][0] = t14 * idet;
	ret[3][1] = (n13 * n24 * n31 - n14 * n23 * n31 + n14 * n21 * n33 - n11 * n24 * n33 - n13 * n21 * n34 + n11 * n23 * n34) * idet;
	ret[3][2] = (n14 * n22 * n31 - n12 * n24 * n31 - n14 * n21 * n32 + n11 * n24 * n32 + n12 * n21 * n34 - n11 * n22 * n34) * idet;
	ret[3][3] = (n12 * n23 * n31 - n13 * n22 * n31 + n13 * n21 * n32 - n11 * n23 * n32 - n12 * n21 * n33 + n11 * n22 * n33) * idet;

	return ret;
}

float3x3 Inverse(float3x3 m)
{
	float n11 = m[0][0], n12 = m[1][0], n13 = m[2][0];
	float n21 = m[0][1], n22 = m[1][1], n23 = m[2][1];
	float n31 = m[0][2], n32 = m[1][2], n33 = m[2][2];

	float t11 = n22 * n33 - n23 * n32;
	float t12 = n13 * n32 - n12 * n33;
	float t13 = n12 * n23 - n13 * n22;

	float det = n11 * t11 + n21 * t12 + n31 * t13;
	float idet = 1.0f / det;

	float3x3 ret;

	ret[0][0] = t11 * idet;
	ret[0][1] = (n23 * n31 - n21 * n33) * idet;
	ret[0][2] = (n21 * n32 - n22 * n31) * idet;

	ret[1][0] = t12 * idet;
	ret[1][1] = (n11 * n33 - n13 * n31) * idet;
	ret[1][2] = (n12 * n31 - n11 * n32) * idet;

	ret[2][0] = t13 * idet;
	ret[2][1] = (n13 * n21 - n11 * n23) * idet;
	ret[2][2] = (n11 * n22 - n12 * n21) * idet;

	return ret;
}
