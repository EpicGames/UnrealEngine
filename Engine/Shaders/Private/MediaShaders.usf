// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 MediaShaders.usf: Shaders to convert various media decoder outputs to RGBA
 =============================================================================*/

#include "Common.ush"
#include "ColorUtils.ush"
#include "GammaCorrectionCommon.ush"
#include "TonemapCommon.ush"

/* Vertex shader
=============================================================================*/

void MainVertexShader(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InUV : ATTRIBUTE1,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV : TEXCOORD0)
{
	OutPosition = InPosition;
	OutUV = InUV;
}


/* Pixel shaders
=============================================================================*/

// shader to convert a AYUV texture to RGBA
void AYUVConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float4 AYUV = AYUVConvertUB.Texture.Sample(AYUVConvertUB.Sampler, InUV.xy);

	OutColor.xyz = YuvToRgb(AYUV.zyx, AYUVConvertUB.ColorTransform, AYUVConvertUB.SrgbToLinear);
	OutColor.w = AYUV.w;
}


// shader to convret a Windows Bitmap texture
void BMPConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	InUV.y = 1.0f - InUV.y;
	float2 ScaledUV = InUV * BMPConvertUB.UVScale;
	float4 RGB = BMPConvertUB.Texture.Sample(BMPConvertUB.Sampler, ScaledUV);

	OutColor.xyz = (BMPConvertUB.SrgbToLinear == 1) ? sRGBToLinear(RGB.xyz) : RGB.xyz;
	OutColor.w = RGB.w;
}

#define UE_Color_EEncoding_Linear		1u
#define UE_Color_EEncoding_sRGB			2u
#define UE_Color_EEncoding_ST2084		3u
#define UE_Color_EEncoding_Gamma22		4u
#define UE_Color_EEncoding_BT1886		5u
#define UE_Color_EEncoding_Gamma26		6u
#define UE_Color_EEncoding_Cineon		7u
#define UE_Color_EEncoding_REDLog		8u
#define UE_Color_EEncoding_REDLog3G10	9u
#define UE_Color_EEncoding_SLog1		10u
#define UE_Color_EEncoding_SLog2		11u
#define UE_Color_EEncoding_SLog3		12u
#define UE_Color_EEncoding_AlexaV3LogC	13u
#define UE_Color_EEncoding_CanonLog		14u
#define UE_Color_EEncoding_ProTune		15u
#define UE_Color_EEncoding_VLog			16u
#define UE_Color_EEncoding_HLG				17u

#define ToneMapMethod_None           0u
#define ToneMapMethod_Hable          1u
#define ToneMapMethod_SimpleReinhard 2u

float3 ApplyColorProcessingRGB(float3 Color, float4x4 CSMtx, uint EOTF, uint ToneMapMethod = 0)
{
	// Go linear
	switch(EOTF)
	{
	case UE_Color_EEncoding_sRGB:			Color = sRGBToLinear(Color); break;
	case UE_Color_EEncoding_ST2084:			Color = ST2084ToLinear(Color); break;
	case UE_Color_EEncoding_SLog3:			Color = SLog3ToLinear(Color); break;
	case UE_Color_EEncoding_HLG:			Color = HLGToLinear(Color); break;
	default: break;
	}

	// Adjust colorspace
	float4 Col4 = float4(Color, 1);
	Color.x = dot(CSMtx[0], Col4);
	Color.y = dot(CSMtx[1], Col4);
	Color.z = dot(CSMtx[2], Col4);

	switch(ToneMapMethod)
	{
	case ToneMapMethod_Hable:			Color = HableToneMap(Color);  break;
	case ToneMapMethod_SimpleReinhard:	Color = SimpleReinhardToneMap(Color);  break;
	default: break;
	}

	return Color;
}


float3 ApplyColorProcessing(float3 YUV, float4x4 YUVMtx, float4x4 CSMtx, uint EOTF, uint ToneMapMethod = 0)
{
	float3 Color;
	float4 YUV4 = float4(YUV, 1);
	Color.x = dot(YUVMtx[0], YUV4);
	Color.y = dot(YUVMtx[1], YUV4);
	Color.z = dot(YUVMtx[2], YUV4);

	// Make sure we safely stay above zero
	Color = max(Color, 0.0f);

	return ApplyColorProcessingRGB(Color, CSMtx, EOTF, ToneMapMethod);
}


// shader to convert a NV12 texture to RGBA (real NV12 texture - DX11)
void NV12ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * NV12ConvertUB.UVScale;

	float3 YUV;
	YUV.x = NV12ConvertUB.SRV_Y.Sample(NV12ConvertUB.SamplerB, ScaledUV).x;
	YUV.yz = NV12ConvertUB.SRV_UV.Sample(NV12ConvertUB.SamplerP, ScaledUV).xy;
	YUV.yz = NV12ConvertUB.SwapChroma != 0u ? YUV.zy : YUV.yz;

	// Output with solid alpha
	OutColor = float4(ApplyColorProcessing(YUV, NV12ConvertUB.ColorTransform, NV12ConvertUB.CSTransform, NV12ConvertUB.EOTF, NV12ConvertUB.ToneMapMethod), 1.0f);
}


// shader to convert a NV12 texture to RGBA (NV12 data, packed into a G8 texture; manually addressed)
void NV12ConvertAsBytesPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * NV12ConvertAsBytesUB.UVScale;

	float3 YUV;

	YUV.x = NV12ConvertAsBytesUB.Texture.Sample(NV12ConvertAsBytesUB.SamplerB, ScaledUV).x;

	ScaledUV.y *= 0.5f;
	ScaledUV.y += 2.0f / 3.0f;

	uint UVOffs = uint(InUV.x * NV12ConvertAsBytesUB.OutputWidth / 2.0f) * 2;
	float UPos = (float(UVOffs * NV12ConvertAsBytesUB.UVScale.x) + 0.5f) / NV12ConvertAsBytesUB.OutputWidth;
	float VPos = (float(UVOffs * NV12ConvertAsBytesUB.UVScale.x) + 1.5f) / NV12ConvertAsBytesUB.OutputWidth;

	YUV.y = NV12ConvertAsBytesUB.Texture.Sample(NV12ConvertAsBytesUB.SamplerP, float2(UPos, ScaledUV.y)).x;
	YUV.z = NV12ConvertAsBytesUB.Texture.Sample(NV12ConvertAsBytesUB.SamplerP, float2(VPos, ScaledUV.y)).x;
	YUV.yz = NV12ConvertAsBytesUB.SwapChroma != 0u ? YUV.zy : YUV.yz;

	OutColor = float4(ApplyColorProcessing(YUV, NV12ConvertAsBytesUB.ColorTransform, NV12ConvertAsBytesUB.CSTransform, NV12ConvertAsBytesUB.EOTF, NV12ConvertAsBytesUB.ToneMapMethod), 1.0f);
}


// shader to convert a NV21 texture to RGBA
void NV21ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * NV21ConvertUB.UVScale;

	float3 YUV;
	YUV.x = NV21ConvertUB.Texture.Sample(NV21ConvertUB.SamplerB, ScaledUV).x;

	ScaledUV.y *= 0.5f;
	ScaledUV.y += 2.0f / 3.0f;

	uint UVOffs = uint(ScaledUV.x * NV21ConvertUB.OutputWidth / 2.0f) * 2;
	float UPos = (float(UVOffs * NV21ConvertUB.UVScale.x) + 1.5f) / NV21ConvertUB.OutputWidth;
	float VPos = (float(UVOffs * NV21ConvertUB.UVScale.x) + 0.5f) / NV21ConvertUB.OutputWidth;

	YUV.y = NV21ConvertUB.Texture.Sample(NV21ConvertUB.SamplerP, float2(UPos, ScaledUV.y)).x;
	YUV.z = NV21ConvertUB.Texture.Sample(NV21ConvertUB.SamplerP, float2(VPos, ScaledUV.y)).x;

	OutColor.xyz = YuvToRgb(YUV, NV21ConvertUB.ColorTransform, NV21ConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}

// shader to convert an RGB texture
void RGBConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * RGBConvertUB.UVScale;
	float4 Color = RGBConvertUB.Texture.Sample(RGBConvertUB.Sampler, ScaledUV);

	OutColor = float4(ApplyColorProcessingRGB(Color.xyz, RGBConvertUB.CSTransform, RGBConvertUB.EOTF, RGBConvertUB.ToneMapMethod),
					  dot(RGBConvertUB.CSTransform[3], Color));
}

static const float4 kYCoCgOffsets = float4(-0.50196078431373, -0.50196078431373, 0, 0);

// shader to convert an YCoCg texture (usually in DXT5 format - note: no alpha present; it is part of the encoding)
void YCoCgConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * YCoCgConvertUB.UVScale;

	float4 CoCgSY = YCoCgConvertUB.Texture.Sample(YCoCgConvertUB.Sampler, ScaledUV);

	CoCgSY += kYCoCgOffsets;

	float Scale = (CoCgSY.z * (255.0 / 8.0)) + 1.0;
	float Co = CoCgSY.x / Scale;
	float Cg = CoCgSY.y / Scale;
	float Y = CoCgSY.w;
	float3 Col = float3(Y + Co - Cg, Y + Cg, Y - Co - Cg);

	OutColor = float4(ApplyColorProcessingRGB(Col, YCoCgConvertUB.CSTransform, YCoCgConvertUB.EOTF, YCoCgConvertUB.ToneMapMethod), 1.0f);
}

// shader to convert a luma (Y) and chroma (CbCr) texture to RGBA
void YCbCrConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float3 YUV;
	YUV.x = YCbCrConvertUB.LumaTexture.Sample(YCbCrConvertUB.LumaSampler, InUV.xy).x;
	YUV.yz = YCbCrConvertUB.CbCrTexture.Sample(YCbCrConvertUB.CbCrSampler, InUV.xy).xy;

	OutColor.xyz = YuvToRgb(YUV, YCbCrConvertUB.ColorTransform, YCbCrConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}

// shader to convert a luma (Y) and chroma (CbCr) texture to RGBA using a full 4x4 matrix passed in
void YCbCrConvertPS_4x4Matrix(
    float4 InPosition : SV_POSITION,
    float2 InUV : TEXCOORD0,
    out float4 OutColor : SV_Target0)
{
    float4 YUV;
    YUV.x = YCbCrConvertUB.LumaTexture.Sample(YCbCrConvertUB.LumaSampler, InUV.xy).x;
    YUV.yz = YCbCrConvertUB.CbCrTexture.Sample(YCbCrConvertUB.CbCrSampler, InUV.xy).xy;
    YUV.w = 1.0;

    float4 sRGB = mul(YCbCrConvertUB.ColorTransform, YUV);
    if (YCbCrConvertUB.SrgbToLinear == 0)
    {
        OutColor = sRGB;
    }
    else
    {
        OutColor = float4(sRGBToLinear(sRGB.xyz), 1.0);
    }
}


// shader to convert a UYVY texture to RGBA
void UYVYConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float4 UYVYB = UYVYConvertUB.Texture.Sample(UYVYConvertUB.SamplerB, float2(InUV.x, InUV.y));
	float4 UYVYP = UYVYConvertUB.Texture.Sample(UYVYConvertUB.SamplerP, float2(InUV.x, InUV.y));

	float Pos = 2.0f * InUV.x * UYVYConvertUB.Width;
	float3 YUV;

	YUV.x = (floor(Pos % 2.0f) == 0) ? UYVYP.y : UYVYP.w;
	YUV.yz = UYVYB.zx;

	OutColor.xyz = YuvToRgb(YUV, UYVYConvertUB.ColorTransform, UYVYConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}


// shader to convert YUV v216/2yuv/vuy2 (YUV 422) to RGB
/*
	Input data os expected as RGBA16 data [representing the Cb:Y0:Cr:Y1 or Y0:Cb:Y1:Cr data for each pixel pair]
	(data could represent 10, 12, 14 or 16 bits per component - storage is always 16-bit OR plain old 8-bit)
*/
void YUVv216ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	// Read 422 data from source (1 input texel generates 2 output texels)
	uint3 PosUV = uint3(InUV * float2(YUVv216ConvertUB.OutputDimX * 0.5, YUVv216ConvertUB.OutputDimY), 0u);
	float4 InRaw = YUVv216ConvertUB.YUVTexture.Load(PosUV);

	// Assemble YUV data from input
	InRaw = (YUVv216ConvertUB.IsARGBFmt != 0u) ? InRaw.zyxw : InRaw;								// PF_ARGB8 vs. PF_ABGR16 ordering
	float4 CbY0CrY1 = (YUVv216ConvertUB.IsCbY0CrY1 != 0u) ? InRaw : InRaw.yxwz;						// CbY0CrY1 vs. Y0CbY1Cr ordering
	CbY0CrY1 = (YUVv216ConvertUB.SwapChroma != 0u) ? CbY0CrY1.zyxw : CbY0CrY1;						// Cb/Cr swap
	float3 YUV = (uint(InUV.x * YUVv216ConvertUB.OutputDimX) & 1u) ? CbY0CrY1.wxz : CbY0CrY1.yxz;	// Decode to YCbCr for odd/even pixels

	// Needed for UYVY to RGB
	BRANCH
	if (YUVv216ConvertUB.UseBilinearSampler != 0u)
	{
		float4 InSampled = YUVv216ConvertUB.YUVTexture.Sample(YUVv216ConvertUB.SamplerB, InUV);
		InSampled = (YUVv216ConvertUB.IsARGBFmt != 0u) ? InSampled.zyxw : InSampled;										// PF_ARGB8 vs. PF_ABGR16 ordering
		float4 CbY0CrY1Sampled = (YUVv216ConvertUB.IsCbY0CrY1 != 0u) ? InSampled : InSampled.yxwz;							// CbY0CrY1 vs. Y0CbY1Cr ordering
		CbY0CrY1Sampled = (YUVv216ConvertUB.SwapChroma != 0u) ? CbY0CrY1Sampled.zyxw : CbY0CrY1Sampled;						// Cb/Cr swap
		float3 YUVSampled = (uint(InUV.x * YUVv216ConvertUB.OutputDimX) & 1u) ? CbY0CrY1Sampled.wxz : CbY0CrY1Sampled.yxz;	// Decode to YCbCr for odd/even pixels
		YUV.yz = YUVSampled.yz;
	}

	OutColor = float4(ApplyColorProcessing(YUV, YUVv216ConvertUB.ColorTransform, YUVv216ConvertUB.CSTransform, YUVv216ConvertUB.EOTF, YUVv216ConvertUB.ToneMapMethod), 1.0f);
}


// shader to convert YUV v210 to RGB
/*
	Input data is expected as R10G10B10A2 data [representing the Cb:Y0:Cr:Y1 data for each pixel pair, A2 is discarded]
*/
void YUVv210ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	uint PosX = uint(InPosition.x);

	uint PixelIndex = Mod(PosX, 6);
	uint3 PosUV = uint3(4 * (PosX / 6), uint(InPosition.y), 0u);

	float3 In0 = YUVv210ConvertUB.YUVTexture.Load(PosUV, int2(0,0)).rgb;	// Cb0, Y0, Cr0
	float3 In1 = YUVv210ConvertUB.YUVTexture.Load(PosUV, int2(1,0)).rgb;	// Y1, Cb1, Y2
	float3 In2 = YUVv210ConvertUB.YUVTexture.Load(PosUV, int2(2,0)).rgb;	// Cr1, Y3, Cb2
	float3 In3 = YUVv210ConvertUB.YUVTexture.Load(PosUV, int2(3,0)).rgb;	// Y4, Cr2, Y5

	float3 YUV;
	if (PixelIndex == 0)
	{
		YUV = In0.yxz;
	}
	else if (PixelIndex == 1)
	{
		YUV = float3(In1.x, In0.xz);
	}
	else if (PixelIndex == 2)
	{
		YUV = float3(In1.zy, In2.x);
	}
	else if (PixelIndex == 3)
	{
		YUV = float3(In2.y, In1.y, In2.x);
	}
	else if (PixelIndex == 4)
	{
		YUV = float3(In3.x, In2.z, In3.y);
	}
	else
	{
		YUV = float3(In3.z, In2.z, In3.y);
	}

	/**
	 * Interpolate chroma for odd pixels within each chroma pair (co-sited left positioning).
	 * Without this, Cb/Cr values show hard steps at every 2-pixel boundary.
	 *
	 * TODO: Same fix could be applied to the other 4:2:2 shaders (UYVYConvertPS, YUVv216ConvertPS, YUY2ConvertPS, YVYUConvertPS).
	 * Their existing hardware bilinear read of the half-width packed texture is off-phase (0.75/0.25 weights rather than 0.5/0.5).
	 */
	const bool bIsOddPixel = (PixelIndex & 1u) != 0u;
	BRANCH
	if (YUVv210ConvertUB.InterpolateChroma != 0u && bIsOddPixel)
	{
		// Branch on which odd pixel (1, 3, or 5) to determine which chroma pair to blend toward.
		float2 NextChroma;
		if (PixelIndex < 2u)
		{
			// Pixel 1: blend chroma pair 0 toward pair 1
			NextChroma = float2(In1.y, In2.x); // Cb1, Cr1
		}
		else if (PixelIndex < 4u)
		{
			// Pixel 3: blend chroma pair 1 toward pair 2
			NextChroma = float2(In2.z, In3.y); // Cb2, Cr2
		}
		else
		{
			// Pixel 5: blend chroma pair 2 toward next group's pair 0 (requires reading ahead).
			BRANCH
			if (PosX + 1u < YUVv210ConvertUB.OutputDimX)
			{
				// Within bounds: read next group's chroma.
				float3 In0Next = YUVv210ConvertUB.YUVTexture.Load(PosUV, int2(4, 0)).rgb;
				NextChroma = float2(In0Next.x, In0Next.z);	// Cb0', Cr0'
			}
			else
			{
				// At right boundary: keep current chroma, no blend.
				NextChroma = YUV.yz;
			}
		}
		
		YUV.yz = lerp(YUV.yz, NextChroma, 0.5);
	}

	OutColor = float4(ApplyColorProcessing(YUV, YUVv210ConvertUB.ColorTransform, YUVv210ConvertUB.CSTransform, YUVv210ConvertUB.EOTF, YUVv210ConvertUB.ToneMapMethod), 1.0f);
}


// shader to convert YUV Y416 to RGB
void YUVY416ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float4 AVYU = YUVY416ConvertUB.SRV_Y.Sample(YUVY416ConvertUB.BilinearClampedSamplerUV, InUV);
	AVYU = (YUVY416ConvertUB.bIsARGBFmt != 0u) ? AVYU.zyxw : AVYU;	// PF_ARGB8 vs. PF_ABGR16 ordering

	OutColor = float4(ApplyColorProcessing(AVYU.yzw, YUVY416ConvertUB.ColorTransform, YUVY416ConvertUB.CSTransform, YUVY416ConvertUB.EOTF, YUVY416ConvertUB.ToneMapMethod), AVYU.x);
}


// shader to convert ARGB16 big endian to RGB
void ARGB16BigConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	uint4 InRaw = ARGB16BigConvertUB.Texture.Load(uint3(ARGB16BigConvertUB.OutputWidth * InUV.x, ARGB16BigConvertUB.OutputHeight * InUV.y, 0));
	InRaw = (InRaw >> 8u) | ((InRaw & 0xff) << 8u);
	float4 In = float4(InRaw) * (1.0 / 65535.0);

	float3 Color = In.yzw;

	OutColor = float4(ApplyColorProcessingRGB(Color, ARGB16BigConvertUB.CSTransform, ARGB16BigConvertUB.EOTF, ARGB16BigConvertUB.ToneMapMethod), In.x);
}


// shader to convert three Y, U, and V textures to RGBA
void YUVConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * YUVConvertUB.UVScale;

	float3 YUV;
	YUV.x = YUVConvertUB.YTexture.Sample(YUVConvertUB.YSampler, ScaledUV).x;
	YUV.y = YUVConvertUB.UTexture.Sample(YUVConvertUB.USampler, ScaledUV).x;
	YUV.z = YUVConvertUB.VTexture.Sample(YUVConvertUB.VSampler, ScaledUV).x;

	OutColor.xyz = YuvToRgb(YUV, YUVConvertUB.ColorTransform, YUVConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}


// shader to convert Y, U, and V textures to RGBA
void YUVScaledConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * YUVScaledConvertUB.UVScale;

	float3 YUV;
	YUV.x = YUVScaledConvertUB.YTexture.Sample(YUVScaledConvertUB.YSampler, ScaledUV).x;
	YUV.y = YUVScaledConvertUB.UTexture.Sample(YUVScaledConvertUB.USampler, ScaledUV).x;
	YUV.z = YUVScaledConvertUB.VTexture.Sample(YUVScaledConvertUB.VSampler, ScaledUV).x;

	YUV *= YUVScaledConvertUB.YUVScale;

	OutColor.xyz = YuvToRgb(YUV, YUVScaledConvertUB.ColorTransform, YUVScaledConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}


// shader to convert a YUY2 texture to RGBA
void YUY2ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * YUY2ConvertUB.UVScale;

	float3 YUV;
	YUV.yz = YUY2ConvertUB.Texture.Sample(YUY2ConvertUB.SamplerB, ScaledUV).yw;

	float4 YUY2P = YUY2ConvertUB.Texture.Sample(YUY2ConvertUB.SamplerP, ScaledUV);
	float Pos = InUV.x * YUY2ConvertUB.OutputWidth;
	YUV.x = (floor(Pos % 2.0f) == 0) ? YUY2P.z : YUY2P.x;

	OutColor.xyz = YuvToRgb(YUV, YUY2ConvertUB.ColorTransform, YUY2ConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}

// shader to convert a YVYU texture to RGBA
void YVYUConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float4 YVYUB = YVYUConvertUB.Texture.Sample(YVYUConvertUB.SamplerB, float2(InUV.x, InUV.y));
	float4 YVYUP = YVYUConvertUB.Texture.Sample(YVYUConvertUB.SamplerP, float2(InUV.x, InUV.y));

	float Pos = 2.0f * InUV.x * YVYUConvertUB.Width;
	float3 YUV;

	YUV.x = (floor(Pos % 2.0f) == 0) ? YVYUP.z : YVYUP.x;
	YUV.yz = YVYUB.wy;

	OutColor.xyz = YuvToRgb(YUV, YVYUConvertUB.ColorTransform, YVYUConvertUB.SrgbToLinear);
	OutColor.w = 1.0f;
}

// shader to convert a P010 texture to RGBA (real P010 texture - DX11)
void P010ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * P010ConvertUB.UVScale;

	float3 YUV;
	YUV.x = P010ConvertUB.SRV_Y.Sample(P010ConvertUB.SamplerB, ScaledUV).x;
	YUV.yz = P010ConvertUB.SRV_UV.Sample(P010ConvertUB.SamplerP, ScaledUV).xy;

	OutColor = float4(ApplyColorProcessing(YUV, P010ConvertUB.ColorTransform, P010ConvertUB.CSTransform, P010ConvertUB.EOTF, P010ConvertUB.ToneMapMethod), 1.0f);
}

// shader to convert a P010 texture to RGBA (P010 data, packed into a G16 texture; manually addressed)
void P010ConvertAsUINT16sPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 ScaledUV = InUV * P010ConvertAsUINT16sUB.UVScale;

	float3 YUV;

	YUV.x = P010ConvertAsUINT16sUB.Texture.Sample(P010ConvertAsUINT16sUB.SamplerB, ScaledUV).x;

	ScaledUV.y *= 0.5f;
	ScaledUV.y += 2.0f / 3.0f;

	uint UVOffs = uint(InUV.x * P010ConvertAsUINT16sUB.OutputWidth / 2.0f) * 2;
	float UPos = (float(UVOffs * P010ConvertAsUINT16sUB.UVScale.x) + 0.5f) / P010ConvertAsUINT16sUB.OutputWidth;
	float VPos = (float(UVOffs * P010ConvertAsUINT16sUB.UVScale.x) + 1.5f) / P010ConvertAsUINT16sUB.OutputWidth;

	YUV.y = P010ConvertAsUINT16sUB.Texture.Sample(P010ConvertAsUINT16sUB.SamplerP, float2(UPos, ScaledUV.y)).x;
	YUV.z = P010ConvertAsUINT16sUB.Texture.Sample(P010ConvertAsUINT16sUB.SamplerP, float2(VPos, ScaledUV.y)).x;

	OutColor = float4(ApplyColorProcessing(YUV, P010ConvertAsUINT16sUB.ColorTransform, P010ConvertAsUINT16sUB.CSTransform, P010ConvertAsUINT16sUB.EOTF, P010ConvertAsUINT16sUB.ToneMapMethod), 1.0f);
}


// shader to convert P010 data stored as 3 pane RGB_2101010 to RGBA
void P010_2101010ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	// ES does not allow reuse of the sampler for >1 textures (the shader is unused with that API, though)
#if COMPILER_GLSL_ES3_1==0
	// Adjust incoming UVs for Y / UV read
	float2 ScaledUV_Y = InUV * P010_2101010ConvertUB.UVScaleY;
	float2 ScaledUV_UV = InUV * P010_2101010ConvertUB.UVScaleUV;

	// Find out which component of the read pixel we will need for Y and UV respectively
	int2 Xint = int2(InUV.x * P010_2101010ConvertUB.OutputWidthY, InUV.x * P010_2101010ConvertUB.OutputWidthUV);
	int2 Xlow = Xint % 3;
	float3 SelY  = (Xlow.x == 0) ? float3(1,0,0) : ((Xlow.x == 1) ? float3(0,1,0) : float3(0,0,1));
	float3 SelUV = (Xlow.y == 0) ? float3(1,0,0) : ((Xlow.y == 1) ? float3(0,1,0) : float3(0,0,1));

	// Build YUV input value from the 3 textures
	float3 YUV;
	YUV.x = dot(P010_2101010ConvertUB.SRV_Y.Sample(P010_2101010ConvertUB.SamplerP, ScaledUV_Y).xyz,  SelY);
	YUV.y = dot(P010_2101010ConvertUB.SRV_U.Sample(P010_2101010ConvertUB.SamplerP, ScaledUV_UV).xyz, SelUV);
	YUV.z = dot(P010_2101010ConvertUB.SRV_V.Sample(P010_2101010ConvertUB.SamplerP, ScaledUV_UV).xyz, SelUV);

	OutColor = float4(ApplyColorProcessing(YUV, P010_2101010ConvertUB.ColorTransform, P010_2101010ConvertUB.CSTransform, P010_2101010ConvertUB.EOTF, P010_2101010ConvertUB.ToneMapMethod), 1.0f);
#else
	OutColor = float4(1,0,1,1);
#endif // COMPILER_GLSL_ES3_1
}


// shader to convert RGB to Y component of YUV
void RGB8toY8ConvertPS(float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float OutColor : SV_Target0)
{
	float3 RGB = RGB8toY8ConvertUB.Texture.Sample(RGB8toY8ConvertUB.SamplerP, float2(InUV.x, InUV.y)).rgb;

	float3 TempRGB = RGB;
	if (RGB8toY8ConvertUB.LinearToSrgb == 1)
	{
		TempRGB = LinearToSrgb(RGB);
	}

	// transform rgb to single y/u/v component
	float		TColor = dot(RGB8toY8ConvertUB.ColorTransform.xyz, TempRGB) + RGB8toY8ConvertUB.ColorTransform.w;

	OutColor = clamp(TColor, 0.0f, 1.0f);
}


TextureExternal InExternalTexture;

// shader to read from TextureExternal source
void ReadTextureExternalPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = float2(dot(InUV, ReadTextureExternalUB.ScaleRotation.xy), dot(InUV, ReadTextureExternalUB.ScaleRotation.zw)) + ReadTextureExternalUB.Offset;
	float4 RGBOut = TextureExternalSample(InExternalTexture, ReadTextureExternalUB.SamplerP, UV);
	OutColor = float4(sRGBToLinear(RGBOut.xyz), 1.0);
}

/* Media Capture Pixel shaders
=============================================================================*/

Texture2D InputTexture;
SamplerState InputSampler;
float4x4 ColorTransform;
uint DoLinearToSrgb;
float OnePixelDeltaX;
float PaddingScale;
float InputPixelOffsetX;


// shader to convert from RGB to 8 bits UYVY
void RGB8toUYVY8ConvertPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float3 RGB0 = InputTexture.Sample(InputSampler, float2(UV.x + 0.5f * OnePixelDeltaX, UV.y)).rgb;
	float3 RGB1 = InputTexture.Sample(InputSampler, float2(UV.x + 1.5f * OnePixelDeltaX, UV.y)).rgb;

	float3 YUV0 = RgbToYuv(RGB0, ColorTransform, DoLinearToSrgb);
	float3 YUV1 = RgbToYuv(RGB1, ColorTransform, DoLinearToSrgb);

	OutColor.xyzw = float4(YUV0.z, YUV0.x, YUV0.y, YUV1.x);
}


// shader to convert from RGB to YUV v210
void RGB10toYUVv210ConvertPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out uint4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float X = (UV.x * PaddingScale) - OnePixelDeltaX * 2.5f;

	float3 RGB0 = InputTexture.Sample(InputSampler, float2(X, UV.y)).rgb;
	float3 RGB1 = InputTexture.Sample(InputSampler, float2(X + OnePixelDeltaX, UV.y)).rgb;
	float3 RGB2 = InputTexture.Sample(InputSampler, float2(X + OnePixelDeltaX * 2.0f, UV.y)).rgb;
	float3 RGB3 = InputTexture.Sample(InputSampler, float2(X + OnePixelDeltaX * 3.0f, UV.y)).rgb;
	float3 RGB4 = InputTexture.Sample(InputSampler, float2(X + OnePixelDeltaX * 4.0f, UV.y)).rgb;
	float3 RGB5 = InputTexture.Sample(InputSampler, float2(X + OnePixelDeltaX * 5.0f, UV.y)).rgb;

	uint3 YUV0 = RgbToYuv(RGB0, ColorTransform, DoLinearToSrgb) * 1023;
	uint3 YUV1 = RgbToYuv(RGB1, ColorTransform, DoLinearToSrgb) * 1023;
	uint3 YUV2 = RgbToYuv(RGB2, ColorTransform, DoLinearToSrgb) * 1023;
	uint3 YUV3 = RgbToYuv(RGB3, ColorTransform, DoLinearToSrgb) * 1023;
	uint3 YUV4 = RgbToYuv(RGB4, ColorTransform, DoLinearToSrgb) * 1023;
	uint3 YUV5 = RgbToYuv(RGB5, ColorTransform, DoLinearToSrgb) * 1023;

	uint W0 = (YUV0.z << 20) | (YUV0.x << 10) | YUV0.y;
	uint W1 = (YUV2.x << 20) | (YUV2.y << 10) | YUV1.x;
	uint W2 = (YUV4.y << 20) | (YUV3.x << 10) | YUV2.z;
	uint W3 = (YUV5.x << 20) | (YUV4.z << 10) | YUV4.x;

	OutColor.xyzw = uint4(W0, W1, W2, W3);
}

// Shader to convert from Android's VYU to RGBA
void VYUConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 RotatedUV = float2(dot(InUV, VYUConvertUB.UVRotation.xy), dot(InUV, VYUConvertUB.UVRotation.zw)) + VYUConvertUB.UVRotOffsetAndScale.xy;
	float2 ScaledUV = RotatedUV * VYUConvertUB.UVRotOffsetAndScale.zw;
	float3 YUV = VYUConvertUB.Texture.Sample(VYUConvertUB.Sampler, ScaledUV).gbr;

	// Output with solid alpha
	OutColor = float4(ApplyColorProcessing(YUV, VYUConvertUB.ColorTransform, VYUConvertUB.CSTransform, VYUConvertUB.EOTF, VYUConvertUB.ToneMapMethod), 1.0f);
}

// Following defines must reflect values in EMediaCaptureConversionOperation in MediaOutput
#define INVERT_ALPHA 3
#define SET_ALPHA_ONE 4

// Shader to swizzle rgba textures, set alpha to 1 or invert alpha
void SwizzleRgbPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float4 Color = InputTexture.Sample(InputSampler, UV);
#if CONVERSION_OP == INVERT_ALPHA
	OutColor.rgba = float4(Color.rgb, 1 - Color.a);
#elif CONVERSION_OP == SET_ALPHA_ONE
	OutColor.rgba = float4(Color.rgb, 1.0f);
#else
	OutColor.rgba = Color.rgba;
#endif
}


