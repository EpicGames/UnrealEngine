// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshDecals.usf: Vertex/Hull/Domain/Pixel shader for computing a mesh decal.
=============================================================================*/

#include "Common.ush"

#if SHADING_PATH_MOBILE
	// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer 
	#define MobileSceneTextures MobileBasePass.SceneTextures
	#define EyeAdaptationStruct MobileBasePass
	#define SubstrateStruct MobileBasePass.SubstratePublic
#else
	#define SceneTexturesStruct DecalPass.SceneTextures
	#define EyeAdaptationStruct DecalPass
	#define SubstrateStruct DecalPass.SubstratePublic
#endif

#include "Substrate/Substrate.ush"
#include "DecalCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"


#if PIXELSHADER

#define MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER (SUBSTRATE_ENABLED && (SHADING_PATH_MOBILE || SUBSTRATE_GBUFFER_FORMAT==0)) // When using forward shading, the base pass pixel shader is used with blending.

#if MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#define SUBSTRATE_MATERIAL_EXPORT_REQUESTED 1
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

#endif // PIXELSHADER

// If virtual texture feedback is enabled then use early depth test so that UAV feedback buffer writes respect the depth test
#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	#if COMPILER_SUPPORTS_DEPTHSTENCIL_EARLYTEST_LATEWRITE
		// If we support early depth test with late write behaviour then use it since we may be using discard, or modifying depth
		#define PIXELSHADER_EARLYDEPTHSTENCIL DEPTHSTENCIL_EARLYTEST_LATEWRITE
	#elif !OUTPUT_PIXEL_DEPTH_OFFSET
		// Otherwise we can only use early depth test if not modifying depth
		// Modifying depth will trigger the slow path where we write feedback to UAV even where depth occluded!
		#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL
	#endif
#endif

// Additional guard to remove all struct usage if empty to avoid platform conflicts.
// This define can be removed if additional data is added to the interpolants struct
#define NEED_MESH_DECAL_INTERPOLATORS	USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS

struct FMeshDecalInterpolants
{
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO : TEXCOORD8; // Arbitrary free slot
#endif
};

struct FMeshDecalVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
#if NEED_MESH_DECAL_INTERPOLATORS
	FMeshDecalInterpolants MeshDecalInterpolants;
#endif
	float4	Position		: SV_POSITION;
};

#define VS_OUTPUT_TYPE FMeshDecalVSToPS

#if VERTEXSHADER

/** transform mesh as normal */
void MainVS(
	FVertexFactoryInput Input,
	out VS_OUTPUT_TYPE Output,
	out FStereoVSOutput StereoOutput
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, StereoOutput);

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	Output.MeshDecalInterpolants.PixelPositionExcludingWPO = WorldPosition.xyz;
#endif

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);
	// Isolate instructions used for world position offset on xbox 360, 
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	{
		WorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);
	}
	
	Output.Position = mul(WorldPosition, ResolvedView.TranslatedWorldToClip);
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif

	// Move all geometry a small amount towards the camera to avoid z-fighting.
	// Assuming a reverse z buffer this pushes the near plane a fixed small distance.
	Output.Position.z += ResolvedView.DecalDepthBias;
}

#endif // VERTEXSHADER

#if PIXELSHADER

// is called in MainPS() from PixelShaderOutputCommon.usf
void FPixelShaderInOut_MainPS(
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants,
#if NEED_MESH_DECAL_INTERPOLATORS
	FMeshDecalInterpolants MeshDecalInterpolants,
#endif
	inout FPixelShaderIn In,
	inout FPixelShaderOut Out,
	uint EyeIndex)
{
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(FactoryInterpolants, In.SvPosition);

	FPixelMaterialInputs PixelMaterialInputs;
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(In.SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, In.SvPosition, ScreenPosition, In.bIsFrontFace, TranslatedWorldPosition, MeshDecalInterpolants.PixelPositionExcludingWPO);
	}
#else
	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(In.SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, In.SvPosition, ScreenPosition, In.bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}
#endif

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	FSubstratePixelHeader SubstrateHeader = MaterialParameters.GetFrontSubstrateHeader();
	const float MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
	SubstrateHeader.IrradianceAO.MaterialAO = MaterialAO;

#if MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER
	float3 EmissiveColor = 0.0f.xxx;
	float Opacity = 0.0f;
	FGBufferData DBufferData;
	if (SubstrateHeader.SubstrateTree.BSDFCount > 0)
	{
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, false/*bRoughnessTracking*/);
		const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
		const float3 V = MaterialParameters.CameraVector;
		FExportResult Export = SubstrateMaterialExportOut(
			Settings,
			SubstrateHeader,
			PixelMaterialInputs.GetFrontSubstrateData(),
			V,
			SurfaceWorldNormal,
			MaterialParameters.WorldPosition_CamRelative,
			0.0f /*MobileShadingPathCurvature*/);
		
		// Either propagate the BSDF opacity, or use the one for AlphaComposite (premultiplied alpha)
		float CoverageOverBackground = Export.Coverage;
	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		CoverageOverBackground = GetMaterialOpacity(PixelMaterialInputs);
	#endif 
		Opacity										= CoverageOverBackground;
		EmissiveColor								= Export.EmissiveLuminance;

		FGBufferData DBufferData					= (FGBufferData)0;
		DBufferData.WorldNormal						= Export.WorldNormal;
		DBufferData.BaseColor						= Export.BaseColor;
		DBufferData.Metallic						= Export.Metallic;
		DBufferData.Specular						= Export.Specular;
		DBufferData.Roughness						= Export.Roughness;
		DBufferData.CustomData						= 0;
		DBufferData.IndirectIrradiance				= 0;
		DBufferData.PrecomputedShadowFactors		= 1;
		DBufferData.GBufferAO						= MaterialAO;
		DBufferData.ShadingModelID					= Export.ShadingModelID;
		DBufferData.SelectiveOutputMask				= 0;
		DBufferData.PerObjectGBufferData			= 1;
		DBufferData.DiffuseIndirectSampleOcclusion	= 0;
		DBufferData.Velocity						= 0;

		DecalCommonOutput(In, Out, EmissiveColor, Opacity, DBufferData);
	}
#else
	float Coverage = 0.0;
	if (SubstrateHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;

		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, false/*bRoughnessTracking*/);
		SubstrateHeader.SubstrateUpdateTree(V, Settings);

		// Either propagate the BSDF opacity, or use the one for AlphaComposite (premultiplied alpha)
		float CoverageOverBackground = SubstrateHeader.SubstrateTree.BSDFs[0].Coverage;
	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		CoverageOverBackground = GetMaterialOpacity(PixelMaterialInputs);
	#endif 
		// Since the Convert-To-Decal node forces parameter blending for the entire Substrate tree, the BSDF we are interested in will be in 0.
		const FSubstrateDBuffer DBuffer = DecalCommonOutput(Out, SubstrateHeader, SubstrateHeader.SubstrateTree.BSDFs[0], 1.0f /*CoverageWeight*/, CoverageOverBackground);
		Coverage = DBuffer.Coverage;
	}
	const float Opacity = Coverage;
#endif // MOBILE_SUBSTRATE_USES_BLENDABLE_GBUFFER

#else // SUBSTRATE_ENABLED

	float3 Color = GetMaterialEmissive(PixelMaterialInputs);

	float Opacity = GetMaterialOpacity(PixelMaterialInputs);

	FGBufferData GBufferData = (FGBufferData)0;
	GBufferData.WorldNormal = MaterialParameters.WorldNormal;
	GBufferData.BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	GBufferData.Metallic = GetMaterialMetallic(PixelMaterialInputs);
	GBufferData.Specular = GetMaterialSpecular(PixelMaterialInputs);
	GBufferData.Roughness = GetMaterialRoughness(PixelMaterialInputs);
	GBufferData.CustomData = 0;
	GBufferData.IndirectIrradiance = 0;
	GBufferData.PrecomputedShadowFactors = 1;
	GBufferData.GBufferAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
	GBufferData.ShadingModelID = SHADINGMODELID_DEFAULT_LIT;
	GBufferData.SelectiveOutputMask = 0;
	GBufferData.PerObjectGBufferData = 1;
	GBufferData.DiffuseIndirectSampleOcclusion = 0;
	GBufferData.Velocity = 0;

	DecalCommonOutput(In, Out, Color, Opacity, GBufferData);
#endif // SUBSTRATE_ENABLED

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif
}

void FPixelShaderInOut_MainPS(
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants,
#if NEED_MESH_DECAL_INTERPOLATORS
	FMeshDecalInterpolants MeshDecalInterpolants,
#endif
	inout FPixelShaderIn In,
	inout FPixelShaderOut Out)
{
#if NEED_MESH_DECAL_INTERPOLATORS
	FPixelShaderInOut_MainPS(FactoryInterpolants, MeshDecalInterpolants, In, Out, 0);
#else 
	FPixelShaderInOut_MainPS(FactoryInterpolants, In, Out, 0);
#endif
}

#define PIXELSHADEROUTPUT_MESHDECALPASS NEED_MESH_DECAL_INTERPOLATORS
#define PIXELSHADEROUTPUT_INTERPOLANTS 1
//#define PIXELSHADEROUTPUT_MRT0 (DECAL_RENDERTARGET_COUNT > 0)
//#define PIXELSHADEROUTPUT_MRT1 (DECAL_RENDERTARGET_COUNT > 1)
//#define PIXELSHADEROUTPUT_MRT2 (DECAL_RENDERTARGET_COUNT > 2)
//#define PIXELSHADEROUTPUT_MRT3 (DECAL_RENDERTARGET_COUNT > 3)
//#define PIXELSHADEROUTPUT_MRT4 (DECAL_RENDERTARGET_COUNT > 4)
// all PIXELSHADEROUTPUT_ and "void FPixelShaderInOut_MainPS()" need to be setup before this include
// this include generates the wrapper code to call MainPS()
#include "PixelShaderOutputCommon.ush"

#endif // PIXELSHADER
