// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshDistanceFieldCommon.ush
=============================================================================*/

#pragma once

float DistanceToNearestSurfaceForObject(FDFObjectData DFObjectData, FDFVector3 WorldPosition, float MaxDistance)
{
	const float3 VolumePosition = DFMultiplyAffineDemote(WorldPosition, DFObjectData.WorldToVolume);

	const float3 ClampedVolumePosition = clamp(VolumePosition, -DFObjectData.VolumePositionExtent, DFObjectData.VolumePositionExtent);

	const float3 ToBox = (abs(VolumePosition) - DFObjectData.VolumePositionExtent) * DFObjectData.VolumeToWorldScale;
	const float DistanceToBoxOutside = length(max(0.0f, ToBox));
	const float DistanceToBoxInside = min(0.0f, max3(ToBox.x, ToBox.y, ToBox.z));
	const float DistanceToBox = DistanceToBoxOutside + DistanceToBoxInside;

	float DistanceToOccluder = MaxDistance;

	BRANCH
	if (DistanceToBox < MaxDistance)
	{
		DistanceToOccluder = DistanceToMeshSurfaceStandalone(ClampedVolumePosition, DFObjectData) * DFObjectData.VolumeScale + max(0.0f, DistanceToBox);

		// Don't allow negative SDF distance to stick out of the bounding box
		DistanceToOccluder = max(DistanceToOccluder, DistanceToBox);
	}

	return DistanceToOccluder;
}

float DistanceToNearestSurfaceForObject(uint ObjectIndex, FDFVector3 WorldPosition, float MaxDistance)
{
	FDFObjectData DFObjectData = LoadDFObjectData(ObjectIndex);
	return DistanceToNearestSurfaceForObject(DFObjectData, WorldPosition, MaxDistance);
}
