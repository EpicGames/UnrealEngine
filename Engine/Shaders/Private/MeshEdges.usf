// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * MeshEdges: Wireframe composited over other shaded views (e.g. Lit)
 */

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float4, MSAA_SAMPLE_COUNT> WireframeColorTexture;
Texture2DMS<float, MSAA_SAMPLE_COUNT>  WireframeDepthTexture;
#else
Texture2D WireframeColorTexture;
Texture2D<float> WireframeDepthTexture;
#endif

SCREEN_PASS_TEXTURE_VIEWPORT(Wireframe)

Texture2D DepthTexture;
SamplerState DepthSampler;
SCREEN_PASS_TEXTURE_VIEWPORT(Depth)
float2 DepthTextureJitter;
float4 SampleOffsetArray[MSAA_SAMPLE_COUNT];

SCREEN_PASS_TEXTURE_VIEWPORT(Output)

float Opacity;
float Gamma;

// Return true if A is closer than B
bool IsCloser(float A, float B)
{
	return (A > B);
}

// Compose wireframe texture over shaded rendertarget, predicated on depth test with nearest shaded surface
void ComposeMeshEdgesPS(
	float4 SvPosition : SV_POSITION,
	#if MSAA_SAMPLE_COUNT > 1
	uint TargetSampleIndex : SV_SampleIndex,
	#endif
	out float4 OutColor : SV_Target0,
	out float OutDepth : SV_Depth)
{
	float2 ViewportUV = (SvPosition.xy - Output_ViewportMin) * Output_ViewportSizeInverse;

	float2 WireframeSamplePosition = (ViewportUV * Wireframe_ViewportSize) + Wireframe_ViewportMin;

	float4 WireframeColor;
	float WireframeDeviceZ;
	float ShadedDeviceZ;
	#if MSAA_SAMPLE_COUNT > 1
	{
		WireframeColor = WireframeColorTexture.Load(WireframeSamplePosition, TargetSampleIndex);
		WireframeDeviceZ = WireframeDepthTexture.Load(WireframeSamplePosition, TargetSampleIndex).r;

		const float2 SampleOffset = SampleOffsetArray[TargetSampleIndex].xy;

		float2 ShadedDepthUV = (ViewportUV * Depth_ViewportSize + Depth_ViewportMin + SampleOffset - DepthTextureJitter) * Depth_ExtentInverse;
		ShadedDepthUV = clamp(ShadedDepthUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
		ShadedDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, ShadedDepthUV, 0).r;
	}
	#else
	{
		WireframeColor = WireframeColorTexture.Load(int3(WireframeSamplePosition, 0));
		WireframeDeviceZ = WireframeDepthTexture.Load(int3(WireframeSamplePosition, 0)).r;

		float2 ShadedDepthUV = (ViewportUV * Depth_ViewportSize + Depth_ViewportMin - DepthTextureJitter) * Depth_ExtentInverse;
		ShadedDepthUV = clamp(ShadedDepthUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
		ShadedDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, ShadedDepthUV, 0).r;
	}
	#endif

	if (WireframeDeviceZ == ShadedDeviceZ && WireframeDeviceZ == FarDepthValue)
	{
		discard;
	}

	// Stop z-fighting by biasing wireframe depth towards camera.
	// The bias factor is a manually tuned heuristic based on surface slope and the proximity of the wireframe to the surface.
	float2 ShadedDepthSlope = abs(float2(ddx(ShadedDeviceZ), ddy(ShadedDeviceZ)) / ShadedDeviceZ);
	float DepthRatio = WireframeDeviceZ / ShadedDeviceZ;
	const float SlopeWeight = 2.0;
	float DepthBiasFactor = lerp(0, 1, (ShadedDepthSlope.x + ShadedDepthSlope.y) * SlopeWeight) + max(1, DepthRatio);

	WireframeDeviceZ *= DepthBiasFactor;

	if (IsCloser(ShadedDeviceZ, WireframeDeviceZ))
	{
		discard;
	}
		
	OutDepth = WireframeDeviceZ;
	
	// WireframeColor already has gamma applied, and will get gamma again, so undo the first one here
	WireframeColor.rgb = pow(WireframeColor.rgb, rcp(Gamma));

	OutColor = float4(WireframeColor.rgb, 1);
	OutColor.a *= Opacity;
}
