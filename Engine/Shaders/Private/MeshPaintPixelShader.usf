// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintPixelShader.usf: Mesh texture paint pixel shader
================================================================================*/

#include "Common.ush"

#if MESH_PAINT_USE_PAINTBRUSH
Texture2D s_PaintBrushTexture;
SamplerState s_PaintBrushTextureSampler;
float RotationOffset;

#if MESH_PAINT_ROTATE_TOWARD_DIRECTION
float2 RotationDirection;
#endif
#endif

/** Texture containing a clone of the texture we're currently rendering to. */
Texture2D s_CloneTexture;
SamplerState s_CloneTextureSampler;

float4x4 c_WorldToBrushMatrix;

/** Brush metrics: x = radius, y = falloff range, z = depth, w = depth falloff range */
float4 c_BrushMetrics;

float c_BrushStrength;
float4 c_BrushColor;
float4 c_ChannelFlags;

float c_GenerateMaskFlag;

// @todo MeshPaint: Remove this?
float c_Gamma;



void Main( float4 InPosition : SV_POSITION,
		   float2 InCloneTextureCoordinates : TEXCOORD0,
		   float3 InWorldSpaceVertexPosition : TEXCOORD1,
		   out float4 OutColor : SV_Target0)
{
	// First sample the source values from the clone texture
	float4 OldColor = Texture2DSample(s_CloneTexture, s_CloneTextureSampler, InCloneTextureCoordinates );


	// Brush metrics: x = radius, y = falloff range, z = depth, w = depth falloff range
	float BrushRadius = c_BrushMetrics.x;
	float BrushRadialFalloffRange = c_BrushMetrics.y;
	float BrushDepth = c_BrushMetrics.z;
	float BrushDepthFalloffRange = c_BrushMetrics.w;

	float4x4 WorldToBrushMatrix = c_WorldToBrushMatrix;

	// Project the pixel into the plane of the brush
	float3 WorldSpaceVertexPosition = InWorldSpaceVertexPosition;
	float3 BrushSpaceVertexPosition = mul(float4( WorldSpaceVertexPosition, 1.0f ), WorldToBrushMatrix).xyz;
	float2 BrushSpaceVertexPosition2D = BrushSpaceVertexPosition.xy;

	// All paths will compute amount of paint to apply
	float PaintAmount = 0;

	// Start by using the old color
	float4 NewColor = OldColor;

#if MESH_PAINT_USE_FILL_BUCKET

#if MESH_PAINT_USE_PAINTBRUSH
	// Flood with a texture selected
	float2 LocalTextureCoordinate = InCloneTextureCoordinates;

	// Rotate our local coordinate system
	float RotationAngleRad = (RotationOffset * (PI / 180.0f));
	
	// Convert from 0->1 to -1->1
	LocalTextureCoordinate.xy -= 0.5f;
	LocalTextureCoordinate.xy *= 2.0f;
	
	// We need to scale up the texture depending on the angle so we always do a complete fill
	// Through geometric calculations we find that the rotation angle will require between 1.0->1.4 times the normal size which changes in the same pattern as a sin wave times twice the period
	float LocalRotationFitScale = 1.0f + 0.4f * abs(sin(RotationAngleRad * 2.0f));
	LocalTextureCoordinate.xy /= LocalRotationFitScale;

	float C = cos(RotationAngleRad);
	float S = sin(RotationAngleRad);
	float2x2 RotationMatrix = float2x2(
		float2( C,  S),
		float2(-S,  C));

	LocalTextureCoordinate = mul(RotationMatrix, LocalTextureCoordinate);

	// Convert from -1->1 range to 0->1
	LocalTextureCoordinate.xy /= 2.0f;
	LocalTextureCoordinate.xy += 0.5f;
	
	PaintAmount = 1.0f;
	PaintAmount *= c_BrushStrength;
	
	// Determine an alpha blended color which is somewhere between our old color and our new color
	float4 AlphaBlendedTextureColor = Texture2DSampleLevel(s_PaintBrushTexture, s_PaintBrushTextureSampler, LocalTextureCoordinate, 0);
	AlphaBlendedTextureColor.rgb = lerp(OldColor.rgb, AlphaBlendedTextureColor.rgb, AlphaBlendedTextureColor.a);

	// Apply brush color after we have found our alpha blended color
	float4 BrushColor = AlphaBlendedTextureColor * c_BrushColor;
	NewColor = lerp(OldColor, BrushColor, PaintAmount * c_ChannelFlags);

#else

	// Simple flood
	PaintAmount = 1.0f;
	PaintAmount *= c_BrushStrength;

	float4 BrushColor = c_BrushColor;
	NewColor = lerp(OldColor, BrushColor, PaintAmount * c_ChannelFlags);
#endif

#elif MESH_PAINT_USE_PAINTBRUSH
	// Calculate local UV position of the paint brush
	float2 LocalTextureCoordinate = BrushSpaceVertexPosition2D;

	// Rotate our local coordinate system
	float RotationAngle = (RotationOffset * (PI / 180.0f));
#if MESH_PAINT_ROTATE_TOWARD_DIRECTION
	RotationAngle += atan2(RotationDirection.y, RotationDirection.x);
#endif
	float C = cos(RotationAngle);
	float S = sin(RotationAngle);

	float2x2 RotationMatrix = float2x2(
		float2( C,  S),
		float2(-S,  C));
		
	LocalTextureCoordinate.xy /= BrushRadius;
	LocalTextureCoordinate = mul(RotationMatrix, LocalTextureCoordinate);

	// Convert from -1->1 range to 0->1
	LocalTextureCoordinate.xy /= 2.0f;
	LocalTextureCoordinate.xy += 0.5f;
	

	// Only allow values between 0->1
	if (LocalTextureCoordinate.x >= 0.0f && LocalTextureCoordinate.x <= 1.0f && LocalTextureCoordinate.y >= 0.0f && LocalTextureCoordinate.y <= 1.0f)
	{
		// OK the vertex is overlapping the brush in 2D space, but is it too close or
		// two far (depth wise) to be influenced?
 		float VertexDepthToBrush = abs( BrushSpaceVertexPosition.z );
 		if( VertexDepthToBrush <= BrushDepth )
		{
			// Compute amount of paint to apply
			PaintAmount = 1.0f;

			// For distance falloff, keep it a square instead of a circle
			float DistanceToVertex2D = max(abs(BrushSpaceVertexPosition2D.x), abs(BrushSpaceVertexPosition2D.y));

			{
				// Compute the actual distance
				float InnerBrushRadius = BrushRadius - BrushRadialFalloffRange;
				if( DistanceToVertex2D > InnerBrushRadius )
				{
					float RadialBasedFalloff = ( DistanceToVertex2D - InnerBrushRadius ) / BrushRadialFalloffRange;
					PaintAmount *= 1.0f - RadialBasedFalloff;
				}
			}

			// Apply depth-based falloff
			{
				float InnerBrushDepth = BrushDepth - BrushDepthFalloffRange;
				if( VertexDepthToBrush > InnerBrushDepth )
				{
					float DepthBasedFalloff = ( VertexDepthToBrush - InnerBrushDepth ) / BrushDepthFalloffRange;
					PaintAmount *= 1.0f - DepthBasedFalloff;
				}
			}
			PaintAmount = saturate(PaintAmount);

			PaintAmount *= c_BrushStrength;
			
			// Determine an alpha blended color which is somewhere between our old color and our new color
			float4 AlphaBlendedTextureColor = Texture2DSampleLevel(s_PaintBrushTexture, s_PaintBrushTextureSampler, LocalTextureCoordinate, 0);
			AlphaBlendedTextureColor.rgb = lerp(OldColor.rgb, AlphaBlendedTextureColor.rgb, AlphaBlendedTextureColor.a);

			// Apply brush color after we have found our alpha blended color
			float4 BrushColor = AlphaBlendedTextureColor * c_BrushColor;

			NewColor = lerp(OldColor, BrushColor, PaintAmount * c_ChannelFlags);
		}
	}
#else

	// Is the brush close enough to the vertex to paint?
 	float DistanceToVertex2D = length( BrushSpaceVertexPosition2D );
 	if( DistanceToVertex2D <= BrushRadius )
	{
		// OK the vertex is overlapping the brush in 2D space, but is it too close or
		// two far (depth wise) to be influenced?
 		float VertexDepthToBrush = abs( BrushSpaceVertexPosition.z );
 		if( VertexDepthToBrush <= BrushDepth )
		{
			// Compute amount of paint to apply
			PaintAmount = 1.0f;

			// Apply radial-based falloff
			{
				// Compute the actual distance
				float InnerBrushRadius = BrushRadius - BrushRadialFalloffRange;
				if( DistanceToVertex2D > InnerBrushRadius )
				{
					float RadialBasedFalloff = ( DistanceToVertex2D - InnerBrushRadius ) / BrushRadialFalloffRange;
					PaintAmount *= 1.0f - RadialBasedFalloff;
				}
			}

			// Apply depth-based falloff
			{
				float InnerBrushDepth = BrushDepth - BrushDepthFalloffRange;
				if( VertexDepthToBrush > InnerBrushDepth )
				{
					float DepthBasedFalloff = ( VertexDepthToBrush - InnerBrushDepth ) / BrushDepthFalloffRange;
					PaintAmount *= 1.0f - DepthBasedFalloff;
				}
			}

			PaintAmount *= c_BrushStrength;

			float4 BrushColor = c_BrushColor;

			NewColor.a = lerp(OldColor.a, BrushColor.a, PaintAmount * c_ChannelFlags.a);
			NewColor.r = lerp(OldColor.r, BrushColor.r, PaintAmount * c_ChannelFlags.r);
			NewColor.g = lerp(OldColor.g, BrushColor.g, PaintAmount * c_ChannelFlags.g);
			NewColor.b = lerp(OldColor.b, BrushColor.b, PaintAmount * c_ChannelFlags.b);
		}
	}
#endif

	OutColor = NewColor;

	// Mask will write out 1 for anywhere painted.
	if ( c_GenerateMaskFlag == 1.0 )
	{
		OutColor.rgb = PaintAmount > 0 ? 1 : 0;
	}

	// Gamma correct the output color.
	if( c_Gamma != 1.0 )
	{
		OutColor.rgb = pow( saturate( OutColor.rgb ), c_Gamma );	
	}
}
