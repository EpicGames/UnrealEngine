// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneData.ush"

uint2 GetMeshPaintTextureDescriptor(FPrimitiveSceneData PrimitiveData)
{
	return PrimitiveData.MeshPaintTextureDescriptor;
}

uint GetMeshPaintTextureCoordinateIndex(FPrimitiveSceneData PrimitiveData)
{
	return GetMeshPaintTextureDescriptor(PrimitiveData).y >> 30;
}

bool GetMeshPaintTextureDescriptorIsValid(FPrimitiveSceneData PrimitiveData)
{
	return GetMeshPaintTextureDescriptor(PrimitiveData).x != 0;
}
