// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintVertexShader.usf: Mesh texture paint vertex shader
================================================================================*/

#include "Common.ush"

float4x4 c_Transform;

void Main( float4 InPosition : ATTRIBUTE0,
		   float2 InCloneTextureCoordinates : ATTRIBUTE2,
		   float3 InWorldSpaceVertexPosition : ATTRIBUTE3,
		   out float4 OutPosition : SV_POSITION,
		   out float2 OutCloneTextureCoordinates: TEXCOORD0,
		   out float3 OutWorldSpaceVertexPosition : TEXCOORD1 )
{
	// Position (on texture map, derived from the UVs of the original mesh)
	OutPosition = mul( InPosition, c_Transform );

	// Pass clone texture coordinates through to the pixel shader
	OutCloneTextureCoordinates = InCloneTextureCoordinates;

	// Position of this vertex in world space (original mesh)
	OutWorldSpaceVertexPosition = InWorldSpaceVertexPosition;
}
