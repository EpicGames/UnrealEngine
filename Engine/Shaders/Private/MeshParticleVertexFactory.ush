// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshParticleVertexFactory.usf: Mesh particle vertex factory shader code.
=============================================================================*/

#include "VertexFactoryCommon.ush"

#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION || PASS_NEEDS_PRIMITIVE_VOLUME_BOUNDS)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)

#define USE_PARTICLE_LOCAL_TO_WORLD (NEEDS_PARTICLE_LOCAL_TO_WORLD || NEEDS_INSTANCE_LOCAL_TO_WORLD_PS)
#define USE_PARTICLE_WORLD_TO_LOCAL (NEEDS_PARTICLE_WORLD_TO_LOCAL || NEEDS_INSTANCE_WORLD_TO_LOCAL_PS)

// Only support previous transform for deferred feature level
#define USE_PREVIOUS_TRANSFORM (FEATURE_LEVEL >= FEATURE_LEVEL_SM4)

#if USE_PREVIOUS_TRANSFORM
#if PARTICLE_MESH_INSTANCED
	Buffer<float4> PrevTransformBuffer;
#else
	float4 PrevTransform0;
	float4 PrevTransform1;
	float4 PrevTransform2;
#endif
#endif
	
#if !PARTICLE_MESH_INSTANCED
	float4 Transform1;
	float4 Transform2;
	float4 Transform3;
	float4 SubUVParams;
	float SubUVLerp;
	float4 ParticleDirection;
	float RelativeTime;
	float4 DynamicParameter;
	float4 ParticleColor;
#endif

struct FVertexFactoryInput
{
	float4	Position	: ATTRIBUTE0;
	HALF3_TYPE	TangentX	: ATTRIBUTE1;
	// TangentZ.w contains sign of tangent basis determinant
	HALF4_TYPE	TangentZ	: ATTRIBUTE2;
	HALF4_TYPE	VertexColor : ATTRIBUTE3;

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	float2	TexCoords0 : ATTRIBUTE4;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		float2	TexCoords1 : ATTRIBUTE5;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		float2	TexCoords2 : ATTRIBUTE6;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		float2	TexCoords3 : ATTRIBUTE7;
	#endif
#elif USE_PARTICLE_SUBUVS
	float2	TexCoords0 : ATTRIBUTE4;
#endif

#if PARTICLE_MESH_INSTANCED
	float4 Transform1 : ATTRIBUTE8;
	float4 Transform2 : ATTRIBUTE9;
	float4 Transform3 : ATTRIBUTE10;

	int4 SubUVParams : ATTRIBUTE11;

	/** X: SubUV lerp Y: Particle relative time. */
	float2 SubUVLerpAndRelTime : ATTRIBUTE12;

	#if USE_DYNAMIC_PARAMETERS
		float4 DynamicParameter	: ATTRIBUTE13;
	#endif	//USE_DYNAMIC_PARAMETERS

	float4 ParticleColor : ATTRIBUTE14;

	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity : ATTRIBUTE15;
#endif

/** Optional instance ID for vertex layered rendering */
#if (PARTICLE_MESH_INSTANCED)
	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	uint InstanceId	: SV_InstanceID;
#else
	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
#endif
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()

#if PASS_NEEDS_VERTEX_ID
	uint VertexId : SV_VertexID;
#endif // #if PASS_NEEDS_VERTEX_ID
};

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if USE_PARTICLE_SUBUVS
	float4 SubUV0AndTexCoord0 : TEXCOORD1;
	float4 SubUV1AndLerp : TEXCOORD2;
#else
	#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2] : TEXCOORD0;
	#endif
#endif

#if INTERPOLATE_VERTEX_COLOR
	float4	VertexColor : COLOR0;
#endif

#if NEEDS_PARTICLE_COLOR
	float4	ParticleColor : COLOR1;
#endif

#if USE_DYNAMIC_PARAMETERS
	nointerpolation float4 DynamicParameter : COLOR2;
#endif
	
#if USE_PARTICLE_POSITION
	/** Particle position in camera-centered translated world space */
	float3 ParticleTranslatedWorldPosition	: PARTICLE_POSITION;
#endif

#if USE_PARTICLE_VELOCITY
	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity	: PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_TIME
	/** Relative alive time of the particle */
	float ParticleTime : PARTICLE_TIME;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	nointerpolation float4 ParticleToWorld[3] : PARTICLE_LOCAL_TO_WORLD;
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	nointerpolation float4 WorldToParticle[3] : PARTICLE_WORLD_TO_LOCAL;
#endif
};

struct FVertexFactoryIntermediates
{
	/** The color of the vertex. */
	float4 VertexColor;

	/** The color of the particle. */
	float4 ParticleColor;

	/** The texture coordinates for the vertex. */
#if NUM_MATERIAL_TEXCOORDS_VERTEX
	float2	TexCoords[NUM_MATERIAL_TEXCOORDS_VERTEX];
#endif 
#if USE_PARTICLE_SUBUVS
	float4 SubUVCoords;
	float SubUVLerp;
#endif
#if USE_DYNAMIC_PARAMETERS
	/** Optional dynamic parameter for the particle. */
	float4 DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity;

	/** Particle position in camera-centered translated world space. */
	float3 ParticleTranslatedWorldPosition;
	
	/** Relative time. */
	float RelativeTime;

	FLWCMatrix ParticleToWorld;
	FLWCInverseMatrix WorldToParticle;

	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

FLWCMatrix GetParticleTransform(FVertexFactoryInput Input)
{
	//Transform is transposed on CPU and position is packed into W. Transpose again to get the correct transform back.
#if PARTICLE_MESH_INSTANCED
	float4x4 Transform4x4 = float4x4(Input.Transform1, Input.Transform2, Input.Transform3, float4(0, 0, 0, 1));
#else
	float4x4 Transform4x4 = float4x4(Transform1, Transform2, Transform3, float4(0, 0, 0, 1));
#endif
	Transform4x4 = transpose(Transform4x4);

	FLWCMatrix FinalMat;
	if ( MeshParticleVF.bUseLocalSpace )
	{
		float3 LWCTile = LWCGetTile(LWCGetOrigin(DFToTileOffset(GetPrimitiveDataFromUniformBuffer().LocalToWorld)));
		FinalMat = MakeLWCMatrix(LWCTile, Transform4x4);
	}
	else
	{
		FinalMat = LWCPromote(Transform4x4);
		FLWCVector3 LWCTileOffset = MakeLWCVector3(MeshParticleVF.LWCTile, 0);
		LWCSetOrigin(FinalMat, LWCAdd(LWCGetOrigin(FinalMat), LWCTileOffset));
	}

	return FinalMat;
}

#if USE_PREVIOUS_TRANSFORM
FLWCMatrix GetParticlePrevTransform(FVertexFactoryInput Input)
{
#if PARTICLE_MESH_INSTANCED
	uint TransformBaseIdx = GetInstanceId(Input.InstanceId) * 3;

	float4x4 PrevTransform4x4 = float4x4(
		PrevTransformBuffer[TransformBaseIdx + 0], 
		PrevTransformBuffer[TransformBaseIdx + 1], 
		PrevTransformBuffer[TransformBaseIdx + 2], 
		float4(0, 0, 0, 1)
		);
#else
	float4x4 PrevTransform4x4 = float4x4(
		PrevTransform0, 
		PrevTransform1, 
		PrevTransform2, 
		float4(0, 0, 0, 1)
		);
#endif
	//Transform is transposed on CPU and position is packed into W. Transpose again to get the correct transform back.
	PrevTransform4x4 = transpose(PrevTransform4x4);

	FLWCMatrix FinalMat;
	if ( MeshParticleVF.bUseLocalSpace )
	{
		float3 LWCTile = LWCGetTile(LWCGetOrigin(DFFastToTileOffset(GetPrimitiveDataFromUniformBuffer().PreviousLocalToWorld)));
		FinalMat = MakeLWCMatrix(LWCTile, PrevTransform4x4);
	}
	else
	{
		FinalMat = LWCPromote(PrevTransform4x4);
		FLWCVector3 LWCTileOffset = MakeLWCVector3(MeshParticleVF.LWCTile, 0);
		LWCSetOrigin(FinalMat, LWCAdd(LWCGetOrigin(FinalMat), LWCTileOffset));
	}
	return FinalMat;
}
#endif

FLWCInverseMatrix GetParticleInverseTransform(FVertexFactoryInput Input)
{
	//Transform is transposed on CPU and position is packed into W. Transpose again to get the correct transform back.
#if PARTICLE_MESH_INSTANCED
	float4x4 Transform = transpose(float4x4(Input.Transform1, Input.Transform2, Input.Transform3, float4(0, 0, 0, 1)));
#else
	float4x4 Transform = transpose(float4x4(Transform1, Transform2, Transform3, float4(0, 0, 0, 1)));
#endif

	float3 Scale2;
	Scale2.x = length2(Transform[0].xyz);
	Scale2.y = length2(Transform[1].xyz);
	Scale2.z = length2(Transform[2].xyz);

	float3 InvNonUniformScale = rsqrt(Scale2);

	float4x4 WorldToLocal = 0;
	WorldToLocal[0].xyz = Transform[0].xyz * Pow2(InvNonUniformScale.x);
	WorldToLocal[1].xyz	= Transform[1].xyz * Pow2(InvNonUniformScale.y);
	WorldToLocal[2].xyz	= Transform[2].xyz * Pow2(InvNonUniformScale.z);
	WorldToLocal[3].w	= 1.0f;
	WorldToLocal		= transpose(WorldToLocal);

	FLWCVector3 LWCTileOffset = MakeLWCVector3(MeshParticleVF.LWCTile, 0);
	FLWCVector3 Translation = LWCPromote(mul(float4(-Transform[3].xyz, 0.0f), WorldToLocal).xyz);
	Translation = LWCAdd(Translation, LWCTileOffset);
	WorldToLocal[3].xyz = Translation.Offset;

	return MakeLWCInverseMatrix(LWCGetTile(Translation), WorldToLocal);
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if USE_PARTICLE_SUBUVS
	#if NUM_MATERIAL_TEXCOORDS
		UNROLL
		for( int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++ )
		{
			Result.TexCoords[CoordinateIndex] = Interpolants.SubUV0AndTexCoord0.zw;
		}
	#endif
	Result.Particle.SubUVCoords[0] = Interpolants.SubUV0AndTexCoord0.xy;
	Result.Particle.SubUVCoords[1] = Interpolants.SubUV1AndLerp.xy;
	Result.Particle.SubUVLerp = Interpolants.SubUV1AndLerp.z;
#elif NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++CoordinateIndex)
	{
		Result.TexCoords[CoordinateIndex * 2] = Interpolants.TexCoords[CoordinateIndex].xy;
		Result.TexCoords[CoordinateIndex * 2 + 1] = Interpolants.TexCoords[CoordinateIndex].wz;
	}
	#if NUM_TEX_COORD_INTERPOLATORS & 1
		Result.TexCoords[NUM_TEX_COORD_INTERPOLATORS - 1] = Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy;
	#endif // #if NUM_TEX_COORD_INTERPOLATORS & 1
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.UnMirrored = TangentToWorld2.w;

#if INTERPOLATE_VERTEX_COLOR
	Result.VertexColor = Interpolants.VertexColor;
#else
	Result.VertexColor = 0;
#endif

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.ParticleColor;
#endif

#if USE_DYNAMIC_PARAMETERS
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter;
#else
	Result.Particle.DynamicParameter = float4(1,1,1,1);
#endif //USE_DYNAMIC_PARAMETERS

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize.xyz = Interpolants.ParticleTranslatedWorldPosition.xyz;
	Result.Particle.TranslatedWorldPositionAndSize.w = 1;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif
#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.ParticleTime;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	//-TODO: LWC Precision Loss
	FLWCMatrix ParticleToWorld = LWCPromote(transpose(float4x4(Interpolants.ParticleToWorld[0], Interpolants.ParticleToWorld[1], Interpolants.ParticleToWorld[2], float4(0.0f, 0.0f, 0.0f, 1.0f))));
	#if NEEDS_PARTICLE_LOCAL_TO_WORLD
		Result.Particle.ParticleToWorld = DFFromTileOffset(ParticleToWorld);
	#endif
	#if NEEDS_INSTANCE_LOCAL_TO_WORLD_PS
		Result.InstanceLocalToWorld = DFFromTileOffset(ParticleToWorld);
	#endif
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	//-TODO: LWC Precision Loss
	FLWCInverseMatrix WorldToParticle = LWCPromoteInverse(transpose(float4x4(Interpolants.WorldToParticle[0], Interpolants.WorldToParticle[1], Interpolants.WorldToParticle[2], float4(0.0f, 0.0f, 0.0f, 1.0f))));
	#if NEEDS_PARTICLE_WORLD_TO_LOCAL
		Result.Particle.WorldToParticle = DFFromTileOffset(WorldToParticle);
	#endif
	#if NEEDS_INSTANCE_WORLD_TO_LOCAL_PS
		Result.InstanceWorldToLocal = DFFromTileOffset(WorldToParticle);
	#endif
#endif

	Result.Particle.MotionBlurFade = 1.0f;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
	Result.TwoSidedSign = 1;

#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Result.WorldVertexNormal_Center = Interpolants.TangentToWorld2_Center.xyz;
#endif

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Intermediates.VertexColor;
	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = TangentBias(Input.TangentZ.xyz);

	Result.Particle.TranslatedWorldPositionAndSize.xyz = Intermediates.ParticleTranslatedWorldPosition.xyz;
	Result.Particle.TranslatedWorldPositionAndSize.w = 1;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
	Result.Particle.RelativeTime = Intermediates.RelativeTime;

#if USE_DYNAMIC_PARAMETERS
	Result.Particle.DynamicParameter = Intermediates.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

	FLWCMatrix ParticleTransform = GetParticleTransform(Input);
	Result.InstanceLocalToWorld = DFFromTileOffset(ParticleTransform);
	Result.InstanceWorldToLocal = DFFromTileOffset(GetParticleInverseTransform(Input));

	Result.Particle.Color = Intermediates.ParticleColor;
	Result.Particle.ParticleToWorld = Result.InstanceLocalToWorld;
	Result.Particle.WorldToParticle = Result.InstanceWorldToLocal;

	Result.TangentToWorld = mul(TangentToLocal, LWCToFloat3x3(ParticleTransform)); 

#if USE_PREVIOUS_TRANSFORM
	Result.PrevFrameLocalToWorld = DFFromTileOffset(GetParticlePrevTransform(Input));
#else
	Result.PrevFrameLocalToWorld = Result.InstanceLocalToWorld;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	for(int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = Intermediates.TexCoords[CoordinateIndex];
	}
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float4 CalcWorldPosition(FVertexFactoryInput Input)
{
	FLWCMatrix Transform = GetParticleTransform(Input);
	float3 WorldPosition = LWCToFloat(LWCAdd(LWCMultiply(Input.Position.xyz, Transform), ResolvedView.TileOffset.PreViewTranslation));
	return float4(WorldPosition, Input.Position.w);
}

float4 CalcPrevWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
#if USE_PREVIOUS_TRANSFORM
	if (MeshParticleVF.PrevTransformAvailable > 0)
	{
		FLWCMatrix PrevTransform = GetParticlePrevTransform(Input);
		float3 PrevWorldPosition = LWCToFloat(LWCAdd(LWCMultiply(Input.Position.xyz, PrevTransform), ResolvedView.TileOffset.PrevPreViewTranslation));
		return float4(PrevWorldPosition, Input.Position.w);
	}
	else
#endif
	{
		float4 Position = CalcWorldPosition(Input);
		Position.xyz -= Intermediates.ParticleVelocity.xyz * Intermediates.ParticleVelocity.w * MeshParticleVF.UseVelocityForMotionBlur;
		return Position;
	}
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	// Swizzle vertex color.
	Intermediates.VertexColor = Input.VertexColor FCOLOR_COMPONENT_SWIZZLE;

	// World position and localtoworld transform.
	FLWCMatrix LocalToWorldMat = GetParticleTransform(Input);

	Intermediates.ParticleToWorld = LocalToWorldMat;
	Intermediates.WorldToParticle = GetParticleInverseTransform(Input);

	FLWCVector3 ParticleWorldPosition = LWCGetOrigin(LocalToWorldMat);
	Intermediates.ParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(ParticleWorldPosition, ResolvedView.TileOffset.PreViewTranslation));

#if PARTICLE_MESH_INSTANCED
	Intermediates.ParticleColor = Input.ParticleColor;
#else
	Intermediates.ParticleColor = ParticleColor;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	Intermediates.TexCoords[0] = Input.TexCoords0;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 1
		Intermediates.TexCoords[1] = Input.TexCoords1;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 2
		Intermediates.TexCoords[2] = Input.TexCoords2;
	#endif
	#if NUM_MATERIAL_TEXCOORDS_VERTEX > 3
		Intermediates.TexCoords[3] = Input.TexCoords3;
	#endif
#endif

#if USE_PARTICLE_SUBUVS
	#if PARTICLE_MESH_INSTANCED
		Intermediates.SubUVCoords.xy = (float2( Input.SubUVParams.x, Input.SubUVParams.y ) + Input.TexCoords0.xy) * MeshParticleVF.SubImageSize.xy;
		Intermediates.SubUVCoords.zw = (float2( Input.SubUVParams.z, Input.SubUVParams.w ) + Input.TexCoords0.xy) * MeshParticleVF.SubImageSize.xy;
		Intermediates.SubUVLerp = Input.SubUVLerpAndRelTime.x;
	#else
		Intermediates.SubUVCoords.xy = (float2( SubUVParams.x, SubUVParams.y ) + Input.TexCoords0.xy) * MeshParticleVF.SubImageSize.xy;
		Intermediates.SubUVCoords.zw = (float2( SubUVParams.z, SubUVParams.w ) + Input.TexCoords0.xy) * MeshParticleVF.SubImageSize.xy;
		Intermediates.SubUVLerp = SubUVLerp.x;
	#endif
#endif 

#if USE_DYNAMIC_PARAMETERS
	#if PARTICLE_MESH_INSTANCED
		Intermediates.DynamicParameter = Input.DynamicParameter;
	#else
		Intermediates.DynamicParameter = DynamicParameter;
	#endif
#endif	//USE_DYNAMIC_PARAMETERS

#if PARTICLE_MESH_INSTANCED
	Intermediates.ParticleVelocity = Input.ParticleVelocity;
	Intermediates.RelativeTime = Input.SubUVLerpAndRelTime.y;
#else
	Intermediates.ParticleVelocity = ParticleDirection;
	Intermediates.RelativeTime = RelativeTime;
#endif

	return Intermediates;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
* this vertex factory will calculate the binormal on-the-fly
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3x3 Result=0;	
	float3 TangentX = TangentBias(Input.TangentX);
	float4 TangentZ = TangentBias(Input.TangentZ);

	// pass-thru the tangent
	Result[0] = TangentX;
	// pass-thru the normal
	Result[2] = TangentZ.xyz;	

	
	// derive the binormal by getting the cross product of the normal and tangent
	Result[1] = cross(Result[2], Result[0]) * TangentZ.w;	
	// Recalculate TangentX off of the other two vectors
	// This corrects quantization error since TangentX was passed in as a quantized vertex input
	// The error shows up most in specular off of a mesh with a smoothed UV seam (normal is smooth, but tangents vary across the seam)
	Result[0] = cross(Result[1], Result[2]) * TangentZ.w;

	return Result;
}

// @return translated world position
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcWorldPosition(Input);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return TangentBias(Input.TangentZ.xyz);
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

void CalcTangentToWorld(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, out float3 TangentToWorld0, out float4 TangentToWorld2)
{
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, Intermediates);

	FLWCMatrix Transform = GetParticleTransform(Input);
	float3x3 TangentToWorld = mul(TangentToLocal, LWCToFloat3x3(Transform));

	// Normalize to remove scaling. Incorrect but faster than computing the inverse-transpose.
	TangentToWorld0 = normalize(TangentToWorld[0]);
	TangentToWorld2 = float4(normalize(TangentToWorld[2]), TangentBias(Input.TangentZ.w) * GetPrimitive_DeterminantSign_FromFlags(Primitive.Flags));
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

#if USE_PARTICLE_SUBUVS
	Interpolants.SubUV0AndTexCoord0.xy = Intermediates.SubUVCoords.xy;
	Interpolants.SubUV1AndLerp.xy = Intermediates.SubUVCoords.zw;
	Interpolants.SubUV1AndLerp.zw = Intermediates.SubUVLerp.xx;

	#if NUM_MATERIAL_TEXCOORDS
		float2 CustomizedUVs[NUM_MATERIAL_TEXCOORDS];
		GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
		Interpolants.SubUV0AndTexCoord0.zw = CustomizedUVs[0];
	#else
		Interpolants.SubUV0AndTexCoord0.zw = 0;
	#endif
#elif NUM_TEX_COORD_INTERPOLATORS
	// Ensure the unused components of the last packed texture coordinate are initialized.
	Interpolants.TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2 - 1] = 0;

	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++CoordinateIndex)
	{
		//float4 TexCoords[(2+1)/2] : TEXCOORD0;
		// float2 CustomizedUVs[2];
		Interpolants.TexCoords[CoordinateIndex].xy = CustomizedUVs[CoordinateIndex * 2];
		Interpolants.TexCoords[CoordinateIndex].wz = CustomizedUVs[CoordinateIndex * 2 + 1];
	}
	#if NUM_TEX_COORD_INTERPOLATORS & 1
		Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy = CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS - 1];
	#endif // #if NUM_TEX_COORD_INTERPOLATORS & 1
#endif


	Interpolants.TangentToWorld0.w = 0;
	CalcTangentToWorld(Input, Intermediates, Interpolants.TangentToWorld0.xyz, Interpolants.TangentToWorld2);
#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	Interpolants.TangentToWorld2_Center = Interpolants.TangentToWorld2;
#endif

#if INTERPOLATE_VERTEX_COLOR
	Interpolants.VertexColor = Intermediates.VertexColor;
#endif

#if NEEDS_PARTICLE_COLOR
	Interpolants.ParticleColor = Intermediates.ParticleColor;
#endif

#if USE_DYNAMIC_PARAMETERS
	Interpolants.DynamicParameter = Intermediates.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

#if USE_PARTICLE_POSITION
	Interpolants.ParticleTranslatedWorldPosition = Intermediates.ParticleTranslatedWorldPosition;
#endif
#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif
#if USE_PARTICLE_TIME
	Interpolants.ParticleTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_LOCAL_TO_WORLD
	//-LWC_TODO: LWC Precision Loss
	float4x4 ParticleToWorldT = transpose(LWCHackToFloat(Intermediates.ParticleToWorld));
	Interpolants.ParticleToWorld[0] = ParticleToWorldT[0];
	Interpolants.ParticleToWorld[1] = ParticleToWorldT[1];
	Interpolants.ParticleToWorld[2] = ParticleToWorldT[2];
#endif

#if USE_PARTICLE_WORLD_TO_LOCAL
	//-LWC_TODO: LWC Precision Loss
	float4x4 WorldToParticleT = transpose(LWCHackToFloat(Intermediates.WorldToParticle));
	Interpolants.WorldToParticle[0] = WorldToParticleT[0];
	Interpolants.WorldToParticle[1] = WorldToParticleT[1];
	Interpolants.WorldToParticle[2] = WorldToParticleT[2];
#endif

	return Interpolants;
}

// @return previous translated world position
float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{	
	return CalcPrevWorldPosition(Input, Intermediates);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

#if NEEDS_VERTEX_FACTORY_INTERPOLATION
	struct FVertexFactoryRayTracingInterpolants
	{
		FVertexFactoryInterpolantsVSToPS InterpolantsVSToPS;
	};

	float2 VertexFactoryGetRayTracingTextureCoordinate( FVertexFactoryRayTracingInterpolants Interpolants )
	{
	#if NUM_MATERIAL_TEXCOORDS
		return Interpolants.InterpolantsVSToPS.TexCoords[0].xy;
	#else // #if NUM_MATERIAL_TEXCOORDS
		return float2(0,0);
	#endif // #if NUM_MATERIAL_TEXCOORDS
	}

	FVertexFactoryInterpolantsVSToPS VertexFactoryAssignInterpolants(FVertexFactoryRayTracingInterpolants Input)
	{
		return Input.InterpolantsVSToPS;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryGetRayTracingInterpolants(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
	{
		FVertexFactoryRayTracingInterpolants Interpolants;
		Interpolants.InterpolantsVSToPS = VertexFactoryGetInterpolantsVSToPS(Input, Intermediates, VertexParameters);
		return Interpolants;
	}

	FVertexFactoryRayTracingInterpolants VertexFactoryInterpolate(FVertexFactoryRayTracingInterpolants a, float aInterp, FVertexFactoryRayTracingInterpolants b, float bInterp)
	{
		FVertexFactoryRayTracingInterpolants O;
	
		// Do we really need to interpolate TangentToWorld2 here? It should be replaced by the
		// interpolated normal from 'whatever' interpolation scheme we're using
	
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld0.xyz);
		INTERPOLATE_MEMBER(InterpolantsVSToPS.TangentToWorld2);
#if INTERPOLATE_VERTEX_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.VertexColor);
#endif

#if NEEDS_PARTICLE_COLOR
		INTERPOLATE_MEMBER(InterpolantsVSToPS.ParticleColor);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
		for (int i = 0; i < (NUM_TEX_COORD_INTERPOLATORS + 1) / 2; ++i)
		{
			INTERPOLATE_MEMBER(InterpolantsVSToPS.TexCoords[i]);
		}
#endif
#if USE_DYNAMIC_PARAMETERS
		INTERPOLATE_MEMBER(InterpolantsVSToPS.DynamicParameter);
#endif	//USE_DYNAMIC_PARAMETERS

		return O;
	}
#endif // #if NEEDS_VERTEX_FACTORY_INTERPOLATION

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#include "VertexFactoryDefaultInterface.ush"