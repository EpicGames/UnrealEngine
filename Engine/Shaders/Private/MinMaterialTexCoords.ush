// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MinMaterialTexCoords.usf: Definition allowing implementing MinMaterialTexCoords.
=============================================================================*/

// Allow some shader types to change the number of coords used. 

#if MIN_MATERIAL_TEXCOORDS > NUM_MATERIAL_TEXCOORDS_VERTEX
	#undef NUM_MATERIAL_TEXCOORDS_VERTEX
	#define NUM_MATERIAL_TEXCOORDS_VERTEX MIN_MATERIAL_TEXCOORDS
#endif

#if MIN_MATERIAL_TEXCOORDS > NUM_MATERIAL_TEXCOORDS
	#undef NUM_MATERIAL_TEXCOORDS
	#define NUM_MATERIAL_TEXCOORDS MIN_MATERIAL_TEXCOORDS
	#undef NUM_TEX_COORD_INTERPOLATORS
	#define NUM_TEX_COORD_INTERPOLATORS (MIN_MATERIAL_TEXCOORDS + NUM_CUSTOM_VERTEX_INTERPOLATORS)
#endif
