// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MiniFontCommon.usf: TinyFont shared functions and structures.
=============================================================================*/

#pragma once

Texture2D MiniFontTexture;

#define NATIVE_CHARACTER_RES 8

// Sub-set of ASCII characters for writing withing shader
#define _0_ 48
#define _1_ 49
#define _2_ 50
#define _3_ 51
#define _4_ 52
#define _5_ 53
#define _6_ 54
#define _7_ 55
#define _8_ 56
#define _9_ 57
#define _A_ 65
#define _B_ 66
#define _C_ 67
#define _D_ 68
#define _E_ 69
#define _F_ 70
#define _G_ 71
#define _H_ 72
#define _I_ 73
#define _J_ 74
#define _K_ 75
#define _L_ 76
#define _M_ 77
#define _N_ 78
#define _O_ 79
#define _P_ 80
#define _Q_ 81
#define _R_ 82
#define _S_ 83
#define _T_ 84
#define _U_ 85
#define _V_ 86
#define _W_ 87
#define _X_ 88
#define _Y_ 89
#define _Z_ 90
#define _MINUS_ 45
#define _COMMA_ 44
#define _DOT_ 46
#define _PLUS_ 43
#define _SPC_ 32
#define _QUESTIONMARK_ 63

// sample from the MiniFont texture
// @param CharacterID - the character to print
// @param Position - the position within the 8x8 character
float SampleMiniFont(int InAsciiCode, uint2 Position)
{
	// Limit ASCII character to the Standard character set (32 - 127)
	const uint TextureCode = clamp(InAsciiCode, 32, 127) - 32;
	return MiniFontTexture.Load(int3(TextureCode * NATIVE_CHARACTER_RES + Position.x, Position.y, 0)).r;
}

// for printf debugging in the shader
// @param LeftTop - is advanced, in pixels
void PrintCharacter(int2 PixelPos, inout float3 OutColor, float3 FontColor, inout int2 LeftTop, int CharacterID)
{
	uint2 Rel = (uint2)(PixelPos - LeftTop);

	// Background: black character as shadow
	FLATTEN if (Rel.x < (NATIVE_CHARACTER_RES + 1) && Rel.y < (NATIVE_CHARACTER_RES + 1))
	{
		OutColor = lerp(OutColor, float3(0.0, 0.0, 0.0), SampleMiniFont(CharacterID, Rel - uint2(1, 1)));
	}
	// Foreground: colored character
	FLATTEN if (Rel.x < (NATIVE_CHARACTER_RES) && Rel.y < (NATIVE_CHARACTER_RES))
	{
		OutColor = lerp(OutColor, FontColor, SampleMiniFont(CharacterID, Rel));
	}

	LeftTop.x += NATIVE_CHARACTER_RES;
}

// only for positive numbers
// @param DigitValue - e.g. 1 for frist digit before period, 10 for second, 0.1 for first digit behind period
uint ExtractDigitFromFloat(float Number, float DigitValue)
{
	uint Temp = (uint)(Number / DigitValue);
	return (Temp % 10) + _0_;
}

// for printf debugging in the shader, has to be positive
// outputs a float number in the form: xxx.yyy
// @param LeftTop - in pixels
void PrintFloat(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	int2 Cursor = LeftTop;

	// before period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 100));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
	// period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _DOT_);
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.1));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.01));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.001));
}

// for printf debugging in the shader, has to be positive
// outputs a float number in the form: xxx.yyy
// @param LeftTop - in pixels
void PrintFloatNoFraction(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	int2 Cursor = LeftTop;

	// before period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10000));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1000));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 100));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
}
void PrintFloatNoFractionLarge(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	int2 Cursor = LeftTop;

	// before period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 100000));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10000));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1000));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 100));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
}
void PrintFloatNoFraction(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number, uint DigitCount)
{
	int2 Cursor = LeftTop;

	uint DigitValue = 1;
	for (uint i = 0; i < DigitCount-1; ++i)
	{
		DigitValue *= 10;
	}
	for (uint j = 0; j < DigitCount; ++j)
	{
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, DigitValue));
		DigitValue /= 10;
	}
}

// for printf debugging in the shader, has to be positive (other abs is taken)
// outputs a float number in the form: xx.yy
// @param LeftTop - in pixels
void PrintSmallFloat(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	Number = abs(Number) + 0.05; // Round up first digit
	int2 Cursor = LeftTop;

	// before period
	FLATTEN
	if (Number >= 10)
	{
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	}
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
	// period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _DOT_);
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.1));
}

void PrintSmallUint(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, uint Number)
{
	int2 Cursor = LeftTop;
	float FloatNumber = float(Number) + 0.05;

	// before period
	FLATTEN
	if (Number >= 10)
	{
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 10));
	}
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(FloatNumber, 1));
}
