// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MissingShaderPixelShader.usf: Outputs the wanted mips.
=============================================================================*/

#include "Common.ush"

#include "DebugViewModeCommon.ush"

// This returns a check board shader. Usefull to figure out that it has maximum variance.


EARLYDEPTHSTENCIL
void Main(
	in FDebugPSIn DebugInputs,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = RETURN_COLOR(float4(UNDEFINED_VALUE, UNDEFINED_VALUE, UNDEFINED_VALUE, .25f));
}
