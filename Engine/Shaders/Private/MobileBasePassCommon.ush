// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassCommon.ush: Base pass definitions used by both vertex and pixel shader
=============================================================================*/

// Just used to trigger MobileBase pass shader recompiles, guid should be regenerated on merge conflicts
#pragma message("UESHADERMETADATA_VERSION C1A9D426-8014-4723-8C69-E32EA8808D15")

#include "BasePassCommon.ush"
#define FogStruct MobileBasePass.Fog
#define PlanarReflectionStruct MobileBasePass.PlanarReflection
#define LFVStruct MobileBasePass

#undef NEEDS_LIGHTMAP_COORDINATE

#define NEEDS_LIGHTMAP_COORDINATE		(LQ_TEXTURE_LIGHTMAP)
#define NEEDS_LIGHTMAP					(NEEDS_LIGHTMAP_COORDINATE)

#ifndef PROJECT_MOBILE_DISABLE_VERTEX_FOG
	#define PROJECT_MOBILE_DISABLE_VERTEX_FOG 0
#endif

// Use vertex fogging for translucency and opaque if project does not use per-pixel fog
// And always use vertex fogging on sky for perf
#define USE_VERTEX_FOG	(!PROJECT_MOBILE_DISABLE_VERTEX_FOG || MATERIAL_IS_SKY || (MATERIAL_ENABLE_TRANSLUCENCY_FOGGING && (MATERIALBLENDING_ANY_TRANSLUCENT || MATERIAL_SHADINGMODEL_SINGLELAYERWATER)))

// Local fog volumes are always evaluated per vertex for now, on mobile, it simply follow VertexFog.
#define LOCAL_FOG_VOLUME_ON_TRANSLUCENT (PROJECT_LOCALFOGVOLUME_APPLYONTRANSLUCENT && USE_VERTEX_FOG && !MATERIAL_IS_SKY)

#define PACK_INTERPOLANTS (USE_VERTEX_FOG && NUM_VF_PACKED_INTERPOLANTS > 0 && (ES3_1_PROFILE))
#define LANDSCAPE_BUG_WORKAROUND (IOS && IS_MOBILE_BASEPASS_VERTEX_SHADER && PACK_INTERPOLANTS)

struct FSharedMobileBasePassInterpolants
{
#if USE_VERTEX_FOG && !PACK_INTERPOLANTS
	#if MOBILE_EMULATION
		float4 VertexFog : TEXCOORD7; // half precision interpolators are causing issues in FXC fallback path as it's used in mobile preview
	#else
		half4 VertexFog : TEXCOORD7;
	#endif
#endif

	float4 PixelPosition	: TEXCOORD8; // xyz = world position, w = clip z

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	float3 PixelPositionExcludingWPO	: TEXCOORD9;
#endif

#if USE_GLOBAL_CLIP_PLANE
	float OutClipDistance : SV_ClipDistance;
#endif

// Keep this dummy at the bottom of the structure for the DXC compiler
// as it currently uses indexing for interpolator naming.  Otherwise
// the VS to PS attribute mapping won't match
#if LANDSCAPE_BUG_WORKAROUND
	half4 DummyInterp		: DUMMY_INTERP;
#endif
};

#define FMobileBasePassInterpolantsVSToPS FSharedMobileBasePassInterpolants
