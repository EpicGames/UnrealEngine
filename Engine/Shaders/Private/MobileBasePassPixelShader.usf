// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassPixelShader.usf: Base pass pixel shader used with forward shading 
=============================================================================*/

#if !MATERIAL_LWC_ENABLED
#define UE_DF_FORCE_FP32_OPS 1
#endif

#include "Common.ush"

// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer 
#define MobileSceneTextures MobileBasePass.SceneTextures
#define EyeAdaptationStruct MobileBasePass

#define ForwardLightStruct MobileBasePass.Forward
#define ReflectionStruct MobileBasePass.ReflectionsParameters
#define SubstrateStruct MobileBasePass.Substrate

#ifndef MOBILE_QL_FORCE_DISABLE_PREINTEGRATEDGF
#define MOBILE_QL_FORCE_DISABLE_PREINTEGRATEDGF 0
#endif

// By default we use translucent fallback for SLW rendering
#ifndef MOBILE_SINGLELAYERWATER_FALLBACK
#define MOBILE_SINGLELAYERWATER_FALLBACK 1
#endif

#define UseBasePassSkylightDiffuse  (1)
#define UseBasePassSkylightSpecular (MobileBasePass.ReflectionsParameters.SkyLightParameters.y)

#define MOBILE_USE_PREINTEGRATED_GF (MATERIAL_USE_PREINTEGRATED_GF && !MOBILE_QL_FORCE_DISABLE_PREINTEGRATEDGF)

//use preintegrated GF lut for simple IBL
#if MOBILE_USE_PREINTEGRATED_GF
#define PreIntegratedGF			MobileBasePass.PreIntegratedGFTexture
#define PreIntegratedGFSampler	MobileBasePass.PreIntegratedGFSampler
#endif

#define APPLY_AO (ENABLE_AMBIENT_OCCLUSION && (MATERIALBLENDING_MASKED || MATERIALBLENDING_SOLID) && !MATERIAL_SHADINGMODEL_UNLIT)

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	#ifdef SINGLE_LAYER_WATER_SHADING_QUALITY
	#undef SINGLE_LAYER_WATER_SHADING_QUALITY
	#endif

	#if !MOBILE_SINGLELAYERWATER_FALLBACK
		// Value must match SINGLE_LAYER_WATER_SHADING_QUALITY_FULL in SingleLayerWaterCommon.ush!
		#define SINGLE_LAYER_WATER_SHADING_QUALITY 0
		// SINGLE_LAYER_WATER_SHADING_QUALITY_FULL normally uses a refraction mask texture which we don't have on mobile
		#define SINGLE_LAYER_WATER_USE_REFRACTION_MASK 0
	#else
		// Value must match SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_BUFFER in SingleLayerWaterCommon.ush!
		#define SINGLE_LAYER_WATER_SHADING_QUALITY 2
	#endif
#endif

// Enable Substrate. This define & include need to be defined before certains includes (i.e., DBufferDecalShared which uses them internally)
#if !MATERIAL_IS_SUBSTRATE && SUBSTRATE_ENABLED
#undef SUBSTRATE_ENABLED
#define SUBSTRATE_ENABLED 0
#endif

#ifndef PROJECT_MOBILE_ENABLE_MOVABLE_SPOTLIGHT_SHADOWS
#define PROJECT_MOBILE_ENABLE_MOVABLE_SPOTLIGHT_SHADOWS 0
#endif

#ifndef MOBILE_QL_FORCE_FULLY_ROUGH
#define MOBILE_QL_FORCE_FULLY_ROUGH 0
#endif
#ifndef MOBILE_QL_FORCE_NONMETAL
#define MOBILE_QL_FORCE_NONMETAL 0
#endif
#ifndef MOBILE_QL_DISABLE_MATERIAL_NORMAL
#define MOBILE_QL_DISABLE_MATERIAL_NORMAL 0
#endif

#define FULLY_ROUGH (MATERIAL_FULLY_ROUGH || MOBILE_QL_FORCE_FULLY_ROUGH)
#define NONMETAL (MATERIAL_NONMETAL || MOBILE_QL_FORCE_NONMETAL)
#define FORCE_VERTEX_NORMAL (MOBILE_QL_DISABLE_MATERIAL_NORMAL)
#define SUPPORT_SPOTLIGHTS_SHADOW (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && PROJECT_MOBILE_ENABLE_MOVABLE_SPOTLIGHT_SHADOWS
// Sky does not write to depth and must not overwrite GBuffer
#define MOBILE_USE_GBUFFER (MOBILE_DEFERRED_SHADING && ((MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !(MATERIAL_SHADINGMODEL_SINGLELAYERWATER && MOBILE_SINGLELAYERWATER_FALLBACK) && !MATERIAL_IS_SKY))

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT || SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE
	#define MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE 1
#else
	#define MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE 0
	#undef MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_DUAL_SRC_BLENDING
	#undef MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	#undef MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_SINGLE_SRC_BLENDING
	#define MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_DUAL_SRC_BLENDING 0
	#define MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING 0
	#define MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_SINGLE_SRC_BLENDING 0
#endif


#if MOBILE_DEFERRED_SHADING && (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE || MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
	#define SceneColorCopyTexture	MobileBasePass.SceneColorCopyTexture
	#define SceneColorCopySampler	MobileBasePass.SceneColorCopySampler
#endif

// SceneDepthAux always enabled for IOS, on other platforms only enabled when in forward and MobileHDR=true
// Sky does not write to depth
#define HAS_SCENE_DEPTH_AUX_OUTPUT (USE_SCENE_DEPTH_AUX && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER && !MATERIAL_IS_SKY)

#if MOBILE_USE_GBUFFER && MOBILE_EXTENDED_GBUFFER
	#define SV_TargetDepthAux SV_Target5	
#elif MOBILE_USE_GBUFFER
	#define SV_TargetDepthAux SV_Target4	
#else
	#define SV_TargetDepthAux SV_Target1
#endif

#define TRANSLUCENCY_NON_DIRECTIONAL (MATERIALBLENDING_ANY_TRANSLUCENT && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL) && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
#define TRANSLUCENCY_SH_LIGHTING (MATERIAL_SHADINGMODEL_DEFAULT_LIT || MATERIAL_SHADINGMODEL_SUBSURFACE) && (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE) && !TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#include "SHCommon.ush"
#include "/Engine/Generated/Material.ush"
#include "MobileBasePassCommon.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "LightmapCommon.ush"  
#define ENABLE_NDOTL_INTEGRATION 1
#include "MobileLightingCommon.ush"
#include "ShadingModelsMaterial.ush"
#include "ThinTranslucentCommon.ush"

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	#include "SingleLayerWaterShading.ush"
#endif

// SUBSTRATE_TODO: Shared with BasePassPixelShader.usf
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING (MATERIAL_IS_SUBSTRATE && SUBSTRATE_ENABLED && SUBSTRATE_OPAQUE_DEFERRED)

#if SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING || MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING || SUBSTRATE_MATERIAL_EXPORT_EXECUTED
#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#endif
#if SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING
#include "/Engine/Private/Substrate/SubstrateMobileForwardLighting.ush"
#endif

#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING || SUBSTRATE_MATERIAL_EXPORT_EXECUTED || SUBSTRATE_GBUFFER_FORMAT==0
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

#if MATERIAL_SHADINGMODEL_HAIR
#ifndef USE_HAIR_COMPLEX_TRANSMITTANCE
#define USE_HAIR_COMPLEX_TRANSMITTANCE 0
#endif
#include "HairStrands/HairStrandsEnvironmentLightingCommon.ush"
#endif

half3 FrameBufferBlendOp(half4 Source)
{
	half4 Dest = half4 (0,0,0,0);

#if MATERIALBLENDING_SOLID
	return Source.rgb;
#elif MATERIALBLENDING_MASKED
	return Source.rgb;
// AlphaComposite will set both MATERIALBLENDING_TRANSLUCENT and MATERIALBLENDING_ALPHACOMPOSITE defines
// so ensure  MATERIALBLENDING_ALPHACOMPOSITE gets first in line
#elif MATERIALBLENDING_ALPHACOMPOSITE
	return Source.rgb + (Dest.rgb*(1.0 - Source.a));
// AlphaHoldout will set both MATERIALBLENDING_TRANSLUCENT and MATERIALBLENDING_ALPHAHOLDOUT defines
// so ensure  MATERIALBLENDING_ALPHAHOLDOUT gets first in line
#elif MATERIALBLENDING_ALPHAHOLDOUT
	return (Dest.rgb*(1.0 - Source.a));
#elif MATERIALBLENDING_TRANSLUCENT
	return (Source.rgb*Source.a) + (Dest.rgb*(1.0 - Source.a));
#elif MATERIALBLENDING_ADDITIVE
	return Source.rgb + Dest.rgb;
#elif MATERIALBLENDING_MODULATE
	return Source.rgb * Dest.rgb;
#endif
}

#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	#if VULKAN_PROFILE
		#define FramebufferFetchColor0 VulkanSubpassFetch0
	#elif METAL_ES3_1_PROFILE && !MAC
		#define FramebufferFetchColor0 SubpassFetchRGBA_0
	#elif COMPILER_GLSL_ES3_1
		#define FramebufferFetchColor0 FramebufferFetchES2
	#else
		#error Platform unsupported for programmable blending
	#endif
#endif // MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING

void ApplyPixelDepthOffsetForMobileBasePass(inout FMaterialPixelParameters MaterialParameters, FPixelMaterialInputs PixelMaterialInputs, out float OutDepth)
{
    float PixelDepthOffset = ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);
}

#define USES_PIXEL_DISCARD ((MATERIALBLENDING_MASKED || USE_DITHERED_LOD_TRANSITION) && !EARLY_Z_PASS_ONLY_MATERIAL_MASKING)
// Force early depth_stencil for materials that use VT feedback and don't use pixel discard or modify the pixel depth
#if (MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED) && !(OUTPUT_PIXEL_DEPTH_OFFSET || USES_PIXEL_DISCARD)
	#define PIXELSHADER_EARLYDEPTHSTENCIL EARLYDEPTHSTENCIL	
#else
	#define PIXELSHADER_EARLYDEPTHSTENCIL 	
#endif

void GetSkyLighting(half3 SkyLightColor, half3 WorldNormal, bool bEvaluateBackface, out half3 OutSkyDiffuseLighting, out half3 OutSkySubsurfaceLighting, out half3 OutWaterDiffuseIndirectLuminance)
{
	OutSkyDiffuseLighting = 0;
	OutSkySubsurfaceLighting = 0;
	OutWaterDiffuseIndirectLuminance = 0;

#if ENABLE_SKY_LIGHT
	{
		half3 SkyDiffuse = GetSkySHDiffuseSimple(WorldNormal);
		OutSkyDiffuseLighting = SkyDiffuse * SkyLightColor;

	#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
		OutWaterDiffuseIndirectLuminance += OutSkyDiffuseLighting;
	#endif

	#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
		if (bEvaluateBackface)
		{
			OutSkySubsurfaceLighting = GetSkySHDiffuseSimple(-WorldNormal) * SkyLightColor;
		}
	#endif
	}
#endif
}

void GetPrecomputedIndirectLightingAndSkyLight(VTPageTableResult LightmapVTPageTableResult, FVertexFactoryInterpolantsVSToPS Interpolants, bool bEvaluateBackface, half3 DiffuseDir, half3 SkyLightColor, out half3 OutDiffuseIndirectLighting, out half3 OutSubsurfaceIndirectLighting, out half OutIndirectIrradiance, out half3 OutWaterDiffuseIndirectLuminance)
{
	//To keep IndirectLightingCache conherence with PC, initialize the OutIndirectIrradiance to zero.
	OutIndirectIrradiance = 0;
	OutDiffuseIndirectLighting = 0;
	OutSubsurfaceIndirectLighting = 0;

	// Indirect Diffuse
#if LQ_TEXTURE_LIGHTMAP
	float2 LightmapUV0, LightmapUV1;
	uint LightmapDataIndex;
	GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);

	GetLightMapColorLQ(LightmapVTPageTableResult, LightmapUV0, LightmapUV1, LightmapDataIndex, DiffuseDir, bEvaluateBackface, OutDiffuseIndirectLighting, OutSubsurfaceIndirectLighting);
#elif CACHED_POINT_INDIRECT_LIGHTING
	#if MATERIALBLENDING_MASKED || MATERIALBLENDING_SOLID
		// Take the normal into account for opaque
		FThreeBandSHVectorRGB PointIndirectLighting;
		PointIndirectLighting.R.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[0];
		PointIndirectLighting.R.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[0];
		PointIndirectLighting.R.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[0];

		PointIndirectLighting.G.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[1];
		PointIndirectLighting.G.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[1];
		PointIndirectLighting.G.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[1];

		PointIndirectLighting.B.V0 = IndirectLightingCache.IndirectLightingSHCoefficients0[2];
		PointIndirectLighting.B.V1 = IndirectLightingCache.IndirectLightingSHCoefficients1[2];
		PointIndirectLighting.B.V2 = IndirectLightingCache.IndirectLightingSHCoefficients2[2];

		FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(DiffuseDir, 1);

		// Compute diffuse lighting which takes the normal into account
		OutDiffuseIndirectLighting += max(half3(0, 0, 0), DotSH3(PointIndirectLighting, DiffuseTransferSH));

		#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
		if (bEvaluateBackface)
		{
			FThreeBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH3(-DiffuseDir, 1);
			OutSubsurfaceIndirectLighting += max(half3(0, 0, 0), DotSH3(PointIndirectLighting, SubsurfaceTransferSH));
		}
		#endif
	#else 
		// Non-directional for translucency
		// Ambient terms packed in xyz
		// Already divided by PI and SH ambient on CPU
		OutDiffuseIndirectLighting += IndirectLightingCache.IndirectLightingSHSingleCoefficient.rgb;

		#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
		if (bEvaluateBackface)
		{
			OutSubsurfaceIndirectLighting += IndirectLightingCache.IndirectLightingSHSingleCoefficient.rgb;
		}
		#endif
	#endif
#endif

	// Apply indirect lighting scale while we have only accumulated lightmaps
	OutDiffuseIndirectLighting *= View.PrecomputedIndirectLightingColorScale;
	OutSubsurfaceIndirectLighting *= View.PrecomputedIndirectLightingColorScale;

	half3 SkyDiffuseLighting;
	half3 SkySubsurfaceLighting;
	GetSkyLighting(SkyLightColor, DiffuseDir, bEvaluateBackface, SkyDiffuseLighting, SkySubsurfaceLighting, OutWaterDiffuseIndirectLuminance);

	OutDiffuseIndirectLighting += SkyDiffuseLighting;
	OutSubsurfaceIndirectLighting += SkySubsurfaceLighting;

#if LQ_TEXTURE_LIGHTMAP || CACHED_POINT_INDIRECT_LIGHTING
	OutIndirectIrradiance = Luminance(OutDiffuseIndirectLighting);
#endif
}

PIXELSHADER_EARLYDEPTHSTENCIL
void Main( 
	FVertexFactoryInterpolantsVSToPS Interpolants
	, FMobileBasePassInterpolantsVSToPS BasePassInterpolants
	, FStereoPSInput StereoInput
	, in float4 SvPosition : SV_Position
	OPTIONAL_IsFrontFace
#if MOBILE_USE_GBUFFER
	#if USE_GLES_FBF_DEFERRED
	, out HALF4_TYPE OutProxy : SV_Target0
	#elif MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	, out HALF4_TYPE OutProgrammableBlending : SV_Target0
	#else
	, out HALF4_TYPE OutColor : SV_Target0
	#endif
	, out HALF4_TYPE OutGBufferA : SV_Target1
	, out HALF4_TYPE OutGBufferB : SV_Target2
	, out HALF4_TYPE OutGBufferC : SV_Target3
	#if MOBILE_EXTENDED_GBUFFER
	, out HALF4_TYPE OutGBufferD : SV_Target4
	#endif
#else
	#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_DUAL_SRC_BLENDING
	, out HALF4_TYPE OutColor DUAL_SOURCE_BLENDING_SLOT(0) : SV_Target0
	, out HALF4_TYPE OutColor1 DUAL_SOURCE_BLENDING_SLOT(1) : SV_Target1
	#elif MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	, out HALF4_TYPE OutProgrammableBlending : SV_Target0
	#else
	, out HALF4_TYPE OutColor : SV_Target0
	#endif
#endif
#if HAS_SCENE_DEPTH_AUX_OUTPUT
	, out float OutSceneDepthAux : SV_TargetDepthAux
#endif
#if OUTPUT_PIXEL_DEPTH_OFFSET
	, out float OutDepth : SV_Depth
#endif
	)
{  
#if MOBILE_USE_GBUFFER 
	#if USE_GLES_FBF_DEFERRED && !MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
		half4 OutColor;
	#endif
	#if !MOBILE_EXTENDED_GBUFFER
		half4 OutGBufferD;
	#endif
#endif
#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	half4 OutColor;
	half4 OutColor1;
#endif

	StereoSetupPS(StereoInput);
	const uint EyeIndex = GetEyeIndex(StereoInput);

#if USE_GLOBAL_CLIP_PLANE && !PLATFORM_SUPPORTS_CLIP_DISTANCE
	clip(BasePassInterpolants.OutClipDistance);
#endif

#if PACK_INTERPOLANTS
	float4 PackedInterpolants[NUM_VF_PACKED_INTERPOLANTS];
	VertexFactoryUnpackInterpolants(Interpolants, PackedInterpolants);
#endif

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	{
#if MOBILE_MULTI_VIEW
		MaterialParameters.ViewId = EyeIndex;
#endif
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 WorldPosition = BasePassInterpolants.PixelPosition.xyz;
		float3 WorldPositionExcludingWPO = BasePassInterpolants.PixelPosition.xyz;
		#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
		WorldPositionExcludingWPO = BasePassInterpolants.PixelPositionExcludingWPO;
		#endif
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, WorldPosition, WorldPositionExcludingWPO);

#if FORCE_VERTEX_NORMAL
		// Quality level override of material's normal calculation, can be used to avoid normal map reads etc.
		MaterialParameters.WorldNormal = MaterialParameters.TangentToWorld[2];
		MaterialParameters.ReflectionVector = ReflectionAboutCustomWorldNormal(MaterialParameters, MaterialParameters.WorldNormal, false);
#endif
	}

#if OUTPUT_PIXEL_DEPTH_OFFSET
	ApplyPixelDepthOffsetForMobileBasePass(MaterialParameters, PixelMaterialInputs, OutDepth);
#endif
	  
#if !EARLY_Z_PASS_ONLY_MATERIAL_MASKING
	//Clip if the blend mode requires it.
	GetMaterialCoverageAndClipping(MaterialParameters, PixelMaterialInputs);
#endif

	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	half4 ScreenSpaceShadowMaskSample = SampleScreenSpaceShadowMask(BufferUV, EyeIndex);

	half MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
#if APPLY_AO
	{
		half GatheredAmbientOcclusion = ScreenSpaceShadowMaskSample.a;
		#if ALLOW_STATIC_LIGHTING
			GatheredAmbientOcclusion = lerp(1.0, GatheredAmbientOcclusion, MobileBasePass.AmbientOcclusionStaticFraction);
		#endif
		MaterialAO *= GatheredAmbientOcclusion;
	}
#endif

	half Opacity = GetMaterialOpacity(PixelMaterialInputs);
	
#if !TEMPLATE_USES_SUBSTRATE
	// Store the results in local variables and reuse instead of calling the functions multiple times.
	half3 BaseColor = max(GetMaterialBaseColorRaw(PixelMaterialInputs), 0.0); // UE-226780 Instead of using GetMaterialBaseColor which does saturate, use max() to avoid driver issue on Adreno V@05xx
	half Metallic = GetMaterialMetallic(PixelMaterialInputs);
	half Specular = GetMaterialSpecular(PixelMaterialInputs);
	// The smallest normalized value that can be represented in IEEE 754 (FP16) is 2^-24 = 5.96e-8.
	// The code will make the following computation involving roughness: 1.0 / Roughness^4.
	// Therefore to prevent division by zero on devices that do not support denormals, Roughness^4
	// must be >= 5.96e-8. We will clamp to 0.015625 because 0.015625^4 = 5.96e-8.
	//
	// Note that we also clamp to 1.0 to match the deferred renderer on PC where the roughness is 
	// stored in an 8-bit value and thus automatically clamped at 1.0.
	half Roughness = max(0.015625f, GetMaterialRoughness(PixelMaterialInputs));
	half Anisotropy = GetMaterialAnisotropy(PixelMaterialInputs);
	uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);
	
	// If we don't use this shading model the color should be black (don't generate shader code for unused data, don't do indirectlighting cache lighting with this color).
	float3 SubsurfaceColor = 0;
	// 0..1, SubsurfaceProfileId = int(x * 255)
	float SubsurfaceProfile = 0;

	#if (MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN || MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || MATERIAL_SHADINGMODEL_CLOTH || MATERIAL_SHADINGMODEL_EYE) 
	if (ShadingModelID == SHADINGMODELID_SUBSURFACE || ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN || ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE || ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE || ShadingModelID == SHADINGMODELID_CLOTH || ShadingModelID == SHADINGMODELID_EYE)
	{
		half4 SubsurfaceData = GetMaterialSubsurfaceData(PixelMaterialInputs);

		if (ShadingModelID == SHADINGMODELID_SUBSURFACE || ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN || ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
		{
			SubsurfaceColor = SubsurfaceData.rgb * ResolvedView.DiffuseOverrideParameter.w + ResolvedView.DiffuseOverrideParameter.xyz;
		}
		else if (ShadingModelID == SHADINGMODELID_CLOTH)
		{
			SubsurfaceColor = SubsurfaceData.rgb;
		}

		SubsurfaceProfile = SubsurfaceData.a;
	}
	#endif
#endif // !TEMPLATE_USES_SUBSTRATE


#if TEMPLATE_USES_SUBSTRATE
	// Initialise a Substrate header with normal in registers
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
#if !SUBSTRATE_OPTIMIZED_UNLIT
	SubstratePixelHeader.IrradianceAO.MaterialAO = MaterialAO;
	SubstratePixelHeader.SetCastContactShadow(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW);
	SubstratePixelHeader.SetDynamicIndirectShadowCasterRepresentation(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION);
#endif 
#endif

#if TEMPLATE_USES_SUBSTRATE && SUBSTRATE_INLINE_SINGLELAYERWATER
	SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_SLWATER);

	// Override GBuffer data with Substrate SLW water BSDF to run forward shadfing code.
	// SUBSTRATE_TODO: run the shading through a Substrate path? (by adding a special BSDF?)
	SubstratePixelHeader.SubstrateTree.BSDFs[0].SubstrateSanitizeBSDF();
	FSubstrateBSDF SLWBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];

	half3 BaseColor = SLW_BASECOLOR(SLWBSDF);
	half Metallic = SLW_METALLIC(SLWBSDF);
	half Specular = SLW_SPECULAR(SLWBSDF);
	half Roughness = SLW_ROUGHNESS(SLWBSDF);
	Opacity = SLW_TOPMATERIALOPACITY(SLWBSDF);
	MaterialParameters.WorldNormal = normalize(SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(SLWBSDF)]);
	half Anisotropy = 0.0f;
	uint ShadingModelID = MATERIAL_SHADINGMODEL_SINGLELAYERWATER;
	float3 SubsurfaceColor = 0;
	float SubsurfaceProfile = 0;
#endif

#if USE_DBUFFER && !MATERIALBLENDING_ANY_TRANSLUCENT && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	BRANCH
	if ((GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER) != 0 && View.ShowDecalsMask > 0)
	{
	#if MATERIALDECALRESPONSEMASK
		// Apply decals from the DBuffer.
		uint ValidDBufferTargetMask = GetDBufferTargetMask(uint2(SvPosition.xy)) & MATERIALDECALRESPONSEMASK;

		BRANCH
		if (ValidDBufferTargetMask)
		{
		#if TEMPLATE_USES_SUBSTRATE
			#if SUBSTRATE_INLINE_SHADING && !SUBSTRATE_OPTIMIZED_UNLIT
			const FSubstrateDBuffer SubstrateBufferData = SubstrateGetDBufferData(BufferUV, ValidDBufferTargetMask, EyeIndex);
			ApplyDBufferData(SubstrateBufferData, SubstratePixelHeader, SubstrateData);
			#endif
		#else
			FDBufferData DBufferData = GetDBufferData(BufferUV, ValidDBufferTargetMask, EyeIndex);
			ApplyDBufferData(DBufferData, MaterialParameters.WorldNormal, SubsurfaceColor, Roughness, BaseColor, Metallic, Specular);
		#endif // TEMPLATE_USES_SUBSTRATE
			
			MaterialParameters.ReflectionVector = ReflectionAboutCustomWorldNormal(MaterialParameters, MaterialParameters.WorldNormal, false);
		}
	#endif
	}
#endif


#if !TEMPLATE_USES_SUBSTRATE || SUBSTRATE_INLINE_SINGLELAYERWATER
	const half BaseMaterialCoverageOverWater = Opacity;
	const half WaterVisibility = 1.0 - BaseMaterialCoverageOverWater;
	// Fade out diffuse as this will be handled by the single scattering lighting. when over the water surface.
	// We keep the SpecularColor for sun/water interactions
	half3 WaterDiffuseIndirectLuminance = 0;

	FGBufferData GBuffer = (FGBufferData)0;
	GBuffer.GBufferAO = MaterialAO;
	GBuffer.Depth = MaterialParameters.ScreenPosition.w;

	SetGBufferForShadingModel(
		GBuffer,
		MaterialParameters,
		PixelMaterialInputs,
		Opacity,
		BaseColor,
		Metallic,
		Specular,
		Roughness,
		Anisotropy,
		SubsurfaceColor,
		SubsurfaceProfile,
		0.0f,
		ShadingModelID
	);

	GBuffer.StoredBaseColor = GBuffer.BaseColor;
	GBuffer.StoredMetallic = GBuffer.Metallic;
	GBuffer.StoredSpecular = GBuffer.Specular;

#if MATERIAL_SHADINGMODEL_EYE
	if( GBuffer.ShadingModelID == SHADINGMODELID_EYE )
	{
		GBuffer.Metallic = 0.0;
		#if IRIS_NORMAL
		GBuffer.Specular = 0.25;
		#endif
	}
#endif

#if MOBILE_QL_FORCE_NONMETAL
	GBuffer.DiffuseColor = GBuffer.BaseColor;
	GBuffer.SpecularColor = DielectricSpecularToF0(GBuffer.SpecularColor);
#else
	GBuffer.SpecularColor = ComputeF0(GBuffer.Specular, GBuffer.BaseColor, GBuffer.Metallic);
	GBuffer.DiffuseColor = GBuffer.BaseColor - GBuffer.BaseColor * GBuffer.Metallic;
#endif

#if FULLY_ROUGH
	EnvBRDFApproxFullyRough(GBuffer.DiffuseColor, GBuffer.SpecularColor);
#endif

#if MOBILE_EMULATION && !MOBILE_DEFERRED_LIGHTING
	{
		// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
		GBuffer.DiffuseColor = GBuffer.DiffuseColor * ResolvedView.DiffuseOverrideParameter.w + ResolvedView.DiffuseOverrideParameter.xyz;
		GBuffer.SpecularColor = GBuffer.SpecularColor * ResolvedView.SpecularOverrideParameter.w + ResolvedView.SpecularOverrideParameter.xyz;
	}
#endif

#if MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS
	// Curvature-to-roughness uses derivatives of the WorldVertexNormal, which is incompatible with centroid interpolation because
	// the samples are not uniformly distributed. Therefore we use WorldVertexNormal_Center which is guaranteed to be center interpolated.
	#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
	float GeometricAARoughness = NormalCurvatureToRoughness(MaterialParameters.WorldVertexNormal_Center);
	#else
	float GeometricAARoughness = NormalCurvatureToRoughness(MaterialParameters.TangentToWorld[2].xyz);
	#endif
	GBuffer.Roughness = max(GBuffer.Roughness, GeometricAARoughness);

	#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		GBuffer.CustomData.y = max(GBuffer.CustomData.y, GeometricAARoughness);
	}
	#endif
#endif

	half3 InputBentNormal = MaterialParameters.WorldNormal;

	// Clear Coat Bottom Normal
	#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	BRANCH if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
	{
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
		InputBentNormal = OctahedronToUnitVector(oct1);
	}
	#endif

	const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, InputBentNormal, GetWorldBentNormalZero(MaterialParameters), GBuffer.Roughness, GBuffer.GBufferAO);
	GBuffer.GBufferAO = AOMultiBounce(Luminance(GBuffer.SpecularColor), ShadingOcclusion.SpecOcclusion).g;

#endif // !TEMPLATE_USES_SUBSTRATE || SUBSTRATE_INLINE_SINGLELAYERWATER

	VTPageTableResult LightmapVTPageTableResult = (VTPageTableResult)0.0f;
#if LIGHTMAP_VT_ENABLED
	{
		float2 LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
		LightmapVTPageTableResult = LightmapGetVTSampleInfo(LightmapUV0, LightmapDataIndex, SvPosition.xy);
	}
#endif

#if LIGHTMAP_VT_ENABLED
	// This must occur after CalcMaterialParameters(), which is required to initialize the VT feedback mechanism
	// Lightmap request is always the first VT sample in the shader
	StoreVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback, LightmapVTPageTableResult.PackedRequest, LightmapVTPageTableResult.PendingMips);
#endif

#if TEMPLATE_USES_SUBSTRATE && !SUBSTRATE_INLINE_SINGLELAYERWATER
	
	float3 Color = 0;
	float3 DualBlendSurfaceLuminancePostCoverage = 0.0f;
	float3 DualBlendSurfaceTransmittancePreCoverage = 1.0f;
	float DualBlendSurfaceCoverage = 1.0f;

  #if !SUBSTRATE_OPTIMIZED_UNLIT

	// Static shadow mask
	#if GBUFFER_HAS_PRECSHADOWFACTOR
	{
		// Encode shadow mask only if the shadow mask is entirely non-zero and non-one
	#if WRITES_PRECSHADOWFACTOR_ZERO
		SubstratePixelHeader.SetHasPrecShadowMask(false);
		SubstratePixelHeader.SetZeroPrecShadowMask(true);
	#else
	  #if ALLOW_STATIC_LIGHTING
		const bool bAllZero = all(GBuffer.PrecomputedShadowFactors == 0);
		const bool bAllOne  = all(GBuffer.PrecomputedShadowFactors == 1);
		if (!bAllZero && !bAllOne)
		{
			SubstratePixelHeader.SetHasPrecShadowMask(true);
		}
		else if (bAllZero)
		{
			SubstratePixelHeader.SetHasPrecShadowMask(false);
			SubstratePixelHeader.SetZeroPrecShadowMask(true);
		}
		else if (bAllOne)
	  #endif
		{
			SubstratePixelHeader.SetHasPrecShadowMask(false);
			SubstratePixelHeader.SetZeroPrecShadowMask(false);
		}
	#endif
	}
	#endif



	#if SUBSTRATE_INLINE_SHADING
	SanitizeAndNormalizeSharedLocalBases(SubstratePixelHeader.SharedLocalBases, MaterialParameters.TangentToWorld);
	#endif // SUBSTRATE_INLINE_SHADING

	FLightAccumulator DirectLighting = (FLightAccumulator)0;
	half3 CameraVector = -MaterialParameters.CameraVector;

#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING

#if SUBSTRATE_INLINE_SINGLELAYERWATER==0
	// Need to reset color to make sure Substrate material are only lit using Substrate lighting passes.
	// Except for water doing some specialised and simplified lighting during the base pass
	Color = 0;
#endif

	{
		SubstratePixelHeader.IrradianceAO									= InitIrradianceAndOcclusion();
		SubstratePixelHeader.IrradianceAO.MaterialAO						= MaterialAO;	// This is the material AO, it will be converted to multibounce AO using ShadingOcclusion.SpecOcclusion in the PackSubstrateOut function, also accounting for bent normal.
		SubstratePixelHeader.IrradianceAO.IndirectIrradiance				= 0;			// SUBSTRATE_TODO
		SubstratePixelHeader.IrradianceAO.DiffuseIndirectSampleOcclusion	= 0;			// SUBSTRATE_TODO GetDiffuseIndirectSampleOcclusion(SubstratePixelHeader.SharedLocalBases, MaterialParameters.CameraVector, SvPosition.xy, SubstratePixelHeader.IrradianceAO.MaterialAO);
	}

	// We only rely on this GBuffer structure for the write out to legacy render target such as velocity or precomputed shadow factors
	FGBufferData GBuffer = (FGBufferData)0;
	GBuffer.PrecomputedShadowFactors = GetPrimaryPrecomputedShadowMask(LightmapVTPageTableResult, Interpolants, MaterialParameters);

#if OUTPUT_PIXEL_DEPTH_OFFSET && SUBSTRATE_INLINE_SINGLELAYERWATER==0 && 0	// We use the legacy GBuffer format for now, so this is disabled.
	// When in deferred, opaque materials with pixel depth offset must execute a custom depth test in order to avoid conflicting UAV writes.
	// This is however not done for single layer water as the depth will never match. Note: that can lead overlapping SLW polys to cause flicker on screen.
	if (ManualDepthTestEqual(SvPosition, OutDepth))
#endif
	{
		uint2 PixelPos = uint2(SvPosition.xy);
		FSubstrateSubsurfaceData SSSData = (FSubstrateSubsurfaceData)0;
		FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
		FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;
		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(FULLY_ROUGH, SubstrateStruct.bRoughDiffuse, SubstrateStruct.PeelLayersAboveDepth, SubstrateStruct.bRoughnessTracking);
	#if SUBSTRATE_ADVANCED_DEBUG_ENABLED && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
		Settings.SliceStoringDebugSubstrateTreeData = all(uint2(float2(View.CursorPosition) * View.ViewResolutionFraction) == PixelPos) ? SubstrateStruct.SliceStoringDebugSubstrateTreeDataWithoutMRT : Settings.SliceStoringDebugSubstrateTreeData;
	#endif

		const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
		const float3 V = MaterialParameters.CameraVector;
		FExportResult Export = SubstrateMaterialExportOut(
			Settings,
			SubstratePixelHeader,
			SubstrateData,
			V,
			SurfaceWorldNormal,
			MaterialParameters.WorldPosition_CamRelative,
		#if SHADING_PATH_MOBILE && MATERIAL_SUBSURFACE_PROFILE_USE_CURVATURE
			GetMaterialCustomData0(PixelMaterialInputs)
		#else
			0.0f
		#endif
			);

		GBuffer.GBufferAO = MaterialAO;
		GBuffer.Depth = MaterialParameters.ScreenPosition.w;

		// Patch material parameter to not have to change the function for now
		MaterialParameters.WorldNormal = Export.WorldNormal;
		MaterialParameters.WorldTangent = Export.WorldTangent;
	#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
		MaterialParameters.WorldVertexNormal_Center = Export.WorldNormal;	// SUBSTRATE_TODO: how to evluate it for center?
	#endif
		MaterialParameters.TangentToWorld[2].xyz = Export.WorldTangent;

		SetGBufferForShadingModel(
			GBuffer,
			MaterialParameters,
			PixelMaterialInputs,
			Export.Coverage,		// Opacity
			Export.BaseColor,
			Export.Metallic,
			Export.Specular,
			Export.Roughness,
			Export.Anisotropy,
			Export.SubsurfaceColor,
			Export.SubsurfaceProfileIDFloat,
			0.0f,					// Dither
			Export.ShadingModelID
		);
		// Set custom data from Substrate export after SetGBufferForShadingModel since it access MaterialParameters to set specific data not available with Substrate
		GBuffer.CustomData = Export.CustomData;

		GBuffer.StoredBaseColor = GBuffer.BaseColor;
		GBuffer.StoredMetallic = GBuffer.Metallic;
		GBuffer.StoredSpecular = GBuffer.Specular;

	#if MATERIAL_SHADINGMODEL_EYE
		if( GBuffer.ShadingModelID == SHADINGMODELID_EYE )
		{
			GBuffer.Metallic = 0.0;
			#if IRIS_NORMAL
			GBuffer.Specular = 0.25;
			#endif
		}
	#endif

	#if MOBILE_QL_FORCE_NONMETAL
		GBuffer.DiffuseColor = GBuffer.BaseColor;
		GBuffer.SpecularColor = DielectricSpecularToF0(GBuffer.SpecularColor);
	#else
		GBuffer.SpecularColor = ComputeF0(GBuffer.Specular, GBuffer.BaseColor, GBuffer.Metallic);
		GBuffer.DiffuseColor = GBuffer.BaseColor - GBuffer.BaseColor * GBuffer.Metallic;
	#endif

	#if FULLY_ROUGH
		EnvBRDFApproxFullyRough(GBuffer.DiffuseColor, GBuffer.SpecularColor);
	#endif

	#if MOBILE_EMULATION && !MOBILE_DEFERRED_LIGHTING
		{
			// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
			GBuffer.DiffuseColor = GBuffer.DiffuseColor * ResolvedView.DiffuseOverrideParameter.w + ResolvedView.DiffuseOverrideParameter.xyz;
			GBuffer.SpecularColor = GBuffer.SpecularColor * ResolvedView.SpecularOverrideParameter.w + ResolvedView.SpecularOverrideParameter.xyz;
		}
	#endif

	#if MATERIAL_NORMAL_CURVATURE_TO_ROUGHNESS
		// Curvature-to-roughness uses derivatives of the WorldVertexNormal, which is incompatible with centroid interpolation because
		// the samples are not uniformly distributed. Therefore we use WorldVertexNormal_Center which is guaranteed to be center interpolated.
		#if USE_WORLDVERTEXNORMAL_CENTER_INTERPOLATION
		float GeometricAARoughness = NormalCurvatureToRoughness(MaterialParameters.WorldVertexNormal_Center);
		#else
		float GeometricAARoughness = NormalCurvatureToRoughness(MaterialParameters.TangentToWorld[2].xyz);
		#endif
		GBuffer.Roughness = max(GBuffer.Roughness, GeometricAARoughness);

		#if MATERIAL_SHADINGMODEL_CLEAR_COAT
		if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
		{
			GBuffer.CustomData.y = max(GBuffer.CustomData.y, GeometricAARoughness);
		}
		#endif
	#endif


		half3 InputBentNormal = MaterialParameters.WorldNormal;
		// Clear Coat Bottom Normal
		#if MATERIAL_SHADINGMODEL_CLEAR_COAT
		BRANCH if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT && CLEAR_COAT_BOTTOM_NORMAL)
		{
			const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
			InputBentNormal = OctahedronToUnitVector(oct1);
		}
		#endif
		const float3 WorldBentNormal0 = GetWorldBentNormalZero(MaterialParameters);
		const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, InputBentNormal, WorldBentNormal0, GBuffer.Roughness, GBuffer.GBufferAO);
		GBuffer.GBufferAO = AOMultiBounce(Luminance(GBuffer.SpecularColor), ShadingOcclusion.SpecOcclusion).g;

		half3 SubsurfaceColor = 0;// SUBSTRATE_TODO
		half3 WaterDiffuseIndirectLuminance = 0;
		half IndirectIrradiance = 0;
		half3 DiffuseIndirectLighting = 0;
		half3 SubsurfaceIndirectLighting = 0;

		half3 DiffuseColorForIndirect = GBuffer.DiffuseColor;
		half3 DiffuseDir = ShadingOcclusion.BentNormal;

	#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
		if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
		{
			// Add subsurface energy to diffuse
			//@todo - better subsurface handling for these shading models with skylight and precomputed GI
			DiffuseColorForIndirect += SubsurfaceColor;
		}
	#endif

	#if MATERIAL_SHADINGMODEL_CLOTH
		if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
		{
			DiffuseColorForIndirect += SubsurfaceColor * saturate(GetMaterialCustomData0(PixelMaterialInputs));
		}
	#endif

	#if MATERIAL_SHADINGMODEL_HAIR
		if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			const half3 N = MaterialParameters.WorldNormal;
			const half3 V = MaterialParameters.CameraVector;
			DiffuseColorForIndirect = EvaluateEnvHair(GBuffer, V, N, DiffuseDir);
		}
	#endif

		//half NoL = max(0, dot(GBuffer.WorldNormal, MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.xyz));

		const bool bEvaluateBackface = GetShadingModelRequiresBackfaceLighting(GBuffer.ShadingModelID);

		GetPrecomputedIndirectLightingAndSkyLight(LightmapVTPageTableResult, Interpolants, bEvaluateBackface, DiffuseDir, ResolvedView.SkyLightColor.rgb, DiffuseIndirectLighting, SubsurfaceIndirectLighting, IndirectIrradiance, WaterDiffuseIndirectLuminance);

		// Apply MaterialAO since we don't have the DiffuseIndirectComposite pass on mobile deferred.
		IndirectIrradiance *= GBuffer.GBufferAO;

		half3 DiffuseColor = (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColor) * AOMultiBounce(GBuffer.BaseColor, ShadingOcclusion.DiffOcclusion);

	#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
		// Fade out diffuse as this will be handled by the single scattering lighting in water material.
		// We do this after the just above GetPrecomputedIndirectLightingAndSkyLight to keep ambiant lighting avialable.
		// We also keep the SpecularColor for sun/water interactions.
		GBuffer.DiffuseColor *= BaseMaterialCoverageOverWater;
		DiffuseColor *= BaseMaterialCoverageOverWater;
		DiffuseColorForIndirect *= BaseMaterialCoverageOverWater;
	#endif

		// This is deferred, should od the same for Substrate GBuffer

		FLightAccumulator DirectLighting = (FLightAccumulator)0;
		LightAccumulator_AddSplit(DirectLighting, DiffuseColor, 0.0f, DiffuseColor, 1.0f, false);
	#if MOBILE_USE_GBUFFER
		GBuffer.PerObjectGBufferData = GetPrimitive_PerObjectGBufferData(MaterialParameters.PrimitiveId);
		GBuffer.IndirectIrradiance = IndirectIrradiance * Export.IndirectIrradianceScale;
		MobileEncodeGBuffer(GBuffer, OutGBufferA, OutGBufferB, OutGBufferC, OutGBufferD);
	#endif

		Color = DirectLighting.TotalLight + Export.EmissiveLuminance;
	}
#endif // MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING

#if (SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING) && !MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
	#define SUBSTRATE_DEBUG_ENABLE_LIGHTING 1
	#if SUBSTRATE_DEBUG_ENABLE_LIGHTING
	if (SubstratePixelHeader.ClosureCount > 0)
	{
		#if FULLY_ROUGH
		const bool bForceFullyRough = true;
		#else
		const bool bForceFullyRough = View.RenderingReflectionCaptureMask > 0; // Can mobile capture reflection probe? Is this needed for mobile?
		#endif
		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(bForceFullyRough, SubstrateStruct.bRoughDiffuse, SubstrateStruct.PeelLayersAboveDepth, SubstrateStruct.bRoughnessTracking);


		// TODO: needed? Remove?
		float3 VolumetricLightmapBrickTextureUVs = 0;
		const float SceneDepth = MaterialParameters.ScreenPosition.w;;
		Color = SubstrateMobileForwardLighting(
			EyeIndex,
			SvPosition,
			Settings,
			BasePassInterpolants,
			Interpolants,
			LightmapVTPageTableResult,
			VolumetricLightmapBrickTextureUVs,
			MaterialParameters,
			SceneDepth,
			BufferUV,
			SubstratePixelHeader,
			SubstrateData,
			DualBlendSurfaceTransmittancePreCoverage,
			DualBlendSurfaceCoverage);

	#if SUBSTRATE_TRANSLUCENT_MATERIAL
		DualBlendSurfaceLuminancePostCoverage = Color;
		Color = 0.0f;
		Opacity = 1.0f; // nullify following operation
	#elif SUBSTRATE_FORWARD_SHADING
		// Color unchanged for opaque materials
		Opacity = 1.0f;
	#else
		#error Unhandled Substrate forward shading mode
	#endif
	}
	#endif // SUBSTRATE_DEBUG_ENABLE_LIGHTING
#endif // SUBSTRATE_TRANSLUCENT_FORWARD || SUBSTRATE_FORWARD_SHADING

  #else // SUBSTRATE_OPTIMIZED_UNLIT

	// Unlit BSDF goes through the SubstrateTree to support weighting operations. Technically, layering and mixing could also be supported.
	float3	UnlitSurfaceNormal = 0.0f;
	SubstratePixelHeader.SubstrateUpdateTreeUnlit(
		uint2(SvPosition.xy),
		MaterialParameters.CameraVector,
		SubstrateData,
		DualBlendSurfaceLuminancePostCoverage,
		DualBlendSurfaceCoverage,
		DualBlendSurfaceTransmittancePreCoverage,
		UnlitSurfaceNormal);
	Color = 0.0f;
	Opacity = 1.0f;

  #endif // SUBSTRATE_OPTIMIZED_UNLIT

	half4 Fogging = half4(0, 0, 0, 1);
	#if USE_VERTEX_FOG
	{
	  #if PACK_INTERPOLANTS
		Fogging = PackedInterpolants[0];
	  #else
		Fogging = BasePassInterpolants.VertexFog;
	  #endif
	}
	#endif

	// Compute final value
	{
	#if SUBSTRATE_OPAQUE_MATERIAL && SUBSTRATE_OPTIMIZED_UNLIT
		OutColor = float4(DualBlendSurfaceLuminancePostCoverage, 1);
	#elif SUBSTRATE_OPAQUE_MATERIAL
		OutColor = float4(Color, 1);
	#endif

	#if SUBSTRATE_TRANSLUCENT_MATERIAL

	#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
		DualBlendSurfaceTransmittancePreCoverage= 0.0;
		DualBlendSurfaceCoverage = GetMaterialOpacity(PixelMaterialInputs);
	#endif

		// Add fog luminance according to surfacecoverage and reduce surface luminance according to fog coverage.
		float3 AdjustedDualBlendAdd = DualBlendSurfaceCoverage * Fogging.rgb + Fogging.a * DualBlendSurfaceLuminancePostCoverage;
		// Fade the surface color transmittance out to 1 according to the surface coverage, and take into account the fog coverage to the surface.
		float3 AdjustedDualBlendMul = lerp(1.0f, Fogging.a * DualBlendSurfaceTransmittancePreCoverage, DualBlendSurfaceCoverage);

	#if SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
		// RETURN_COLOR not needed with modulative blending
		half3 FoggedColor = lerp(float3(1, 1, 1), DualBlendSurfaceTransmittancePreCoverage, Fogging.aaa* DualBlendSurfaceCoverage);
		OutColor = half4(FoggedColor, 1.0f);
	#elif SUBSTRATE_BLENDING_ALPHAHOLDOUT
		half AdjustedAlpha = saturate(1 - dot(AdjustedDualBlendMul, half3(1.0f, 1.0f, 1.0f) / 3.0f));
		OutColor = half4(0.0f.xxx, AdjustedAlpha);
	#elif MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE && MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_DUAL_SRC_BLENDING
		OutColor = half4(AdjustedDualBlendAdd, 0.0);
		OutColor1 = half4(AdjustedDualBlendMul, 1.0);
	#else
		// Pre multipled alpha blending
		half AdjustedAlpha = saturate(1 - dot(AdjustedDualBlendMul, half3(1.0f, 1.0f, 1.0f) / 3.0f));
		#if SUBSTRATE_USES_CONVERSION_FROM_LEGACY && MATERIALBLENDING_ADDITIVE
			AdjustedAlpha = 0.0f;
		#endif
		// Pre multipled alpha blending
		OutColor = half4(AdjustedDualBlendAdd, AdjustedAlpha);
	#endif

	#endif // SUBSTRATE_TRANSLUCENT_MATERIAL
	}

#else // TEMPLATE_USES_SUBSTRATE

	half IndirectIrradiance = 0;
	half3 DiffuseIndirectLighting = 0;
	half3 SubsurfaceIndirectLighting = 0;

	half3 DiffuseColorForIndirect = GBuffer.DiffuseColor;
	half3 DiffuseDir = ShadingOcclusion.BentNormal;

#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
	if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		// Add subsurface energy to diffuse
		//@todo - better subsurface handling for these shading models with skylight and precomputed GI
		DiffuseColorForIndirect += SubsurfaceColor;
	}
#endif

#if MATERIAL_SHADINGMODEL_CLOTH
	if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
	{
		DiffuseColorForIndirect += SubsurfaceColor * saturate(GetMaterialCustomData0(PixelMaterialInputs));
	}
#endif

#if MATERIAL_SHADINGMODEL_HAIR
	if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
	{
		const half3 N = MaterialParameters.WorldNormal;
		const half3 V = MaterialParameters.CameraVector;
		DiffuseColorForIndirect = EvaluateEnvHair(GBuffer, V, N, DiffuseDir);
	}
#endif

	half NoL = max(0, dot(GBuffer.WorldNormal, MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.xyz));

	const bool bEvaluateBackface = GetShadingModelRequiresBackfaceLighting(GBuffer.ShadingModelID);

	GetPrecomputedIndirectLightingAndSkyLight(LightmapVTPageTableResult, Interpolants, bEvaluateBackface, DiffuseDir, ResolvedView.SkyLightColor.rgb, DiffuseIndirectLighting, SubsurfaceIndirectLighting, IndirectIrradiance, WaterDiffuseIndirectLuminance);

	// Apply MaterialAO since we don't have the DiffuseIndirectComposite pass on mobile deferred.
	IndirectIrradiance *= GBuffer.GBufferAO;

	half3 DiffuseColor = (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColor) * AOMultiBounce(GBuffer.BaseColor, ShadingOcclusion.DiffOcclusion);

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	// Fade out diffuse as this will be handled by the single scattering lighting in water material.
	// We do this after the just above GetPrecomputedIndirectLightingAndSkyLight to keep ambiant lighting avialable.
	// We also keep the SpecularColor for sun/water interactions.
	GBuffer.DiffuseColor *= BaseMaterialCoverageOverWater;
	DiffuseColor *= BaseMaterialCoverageOverWater;
	DiffuseColorForIndirect *= BaseMaterialCoverageOverWater;
#endif

	FLightAccumulator DirectLighting = (FLightAccumulator)0;
	LightAccumulator_AddSplit(DirectLighting, DiffuseColor, 0.0f, DiffuseColor, 1.0f, false);

	GBuffer.PrecomputedShadowFactors = GetPrimaryPrecomputedShadowMask(LightmapVTPageTableResult, Interpolants, MaterialParameters);

#if MOBILE_USE_GBUFFER
	GBuffer.IndirectIrradiance = IndirectIrradiance;
	GBuffer.PerObjectGBufferData = GetPrimitive_PerObjectGBufferData(MaterialParameters.PrimitiveId);
	MobileEncodeGBuffer(GBuffer, OutGBufferA, OutGBufferB, OutGBufferC, OutGBufferD);
#endif

#if !MOBILE_USE_GBUFFER || MATERIAL_SHADINGMODEL_SINGLELAYERWATER // With SM5 material support, SLW writes to the gbuffer but also still needs to evaluate this forward shading branch.

	half3 CameraVector = -MaterialParameters.CameraVector;

	half4 DynamicShadowFactors = half4(ScreenSpaceShadowMaskSample.rgb, 1.0);
	float DirectionalLightShadow = 1.0f;
	// Directional light
#if !MATERIAL_SHADINGMODEL_UNLIT
	AccumulateDirectionalLighting(GBuffer, MaterialParameters.WorldPosition_CamRelative, CameraVector, MaterialParameters.ScreenPosition, DynamicShadowFactors, 0, DirectionalLightShadow, DirectLighting);
#endif

	float2 LocalPosition = SvPosition.xy - ResolvedView.ViewRectMin.xy;
	uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SvPosition.w, EyeIndex);

#if MATERIALBLENDING_MASKED || MATERIALBLENDING_SOLID || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME || MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	// Reflection IBL
	AccumulateReflection(GBuffer
		, SvPosition
		, CameraVector
		, MaterialParameters.WorldPosition_CamRelative
		, MaterialParameters.ReflectionVector
		, IndirectIrradiance
		, GridIndex
		, DirectLighting);
#endif

	// Local lights
#if (!MATERIAL_SHADINGMODEL_SINGLELAYERWATER && ENABLE_CLUSTERED_LIGHTS)
	{
		const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);

		half4 LocalLightDynamicShadowFactors = 1.0f;

		#if ENABLE_SHADOWMASKTEXTURE
			LocalLightDynamicShadowFactors = DynamicShadowFactors;
		#endif
		
		uint LightingChannelMask = GetPrimitive_LightingChannelMask(MaterialParameters.PrimitiveId);
		AccumulateLightGridLocalLighting(CulledLightGridHeader, GBuffer, MaterialParameters.WorldPosition_CamRelative, CameraVector, EyeIndex, 0, LocalLightDynamicShadowFactors, LightingChannelMask, DirectLighting);
	}
#endif

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	{
		const bool bSeparateMainDirLightLuminance = false;
		float3 SeparatedWaterMainDirLightLuminance = float3(0, 0, 0);

		const bool CameraIsUnderWater = false;	// Fade out the material contribution over to water contribution according to material opacity.
		float3 SunIlluminance = ResolvedView.DirectionalLightColor.rgb * PI;			// times PI because it is divided by PI on CPU (=luminance) and we want illuminance here. 
		float3 WaterDiffuseIndirectIlluminance = WaterDiffuseIndirectLuminance * PI;	// DiffuseIndirectLighting is luminance. So we need to multiply by PI to get illuminance.
#if MOBILE_EMULATION
		SunIlluminance = lerp(SunIlluminance, 0.0f, View.UnlitViewmodeMask);
		WaterDiffuseIndirectIlluminance = lerp(WaterDiffuseIndirectIlluminance, PI, View.UnlitViewmodeMask);
#endif

#if !MOBILE_SINGLELAYERWATER_FALLBACK
		const float4 SLWBufferBilinearUVMinMax = View.BufferSizeAndInvSize.zwzw * (ResolvedView.ViewRectMinAndSize.xyxy + float4(0.55f, 0.55f, ResolvedView.ViewRectMinAndSize.zw - 0.55f));
#endif

		half3 N = GBuffer.WorldNormal;
		half3 V = MaterialParameters.CameraVector;
		half NoV = saturate(abs(dot(N, V)) + 1e-5);
		const float3 EnvBrdf = GetEnvBRDF(GBuffer.SpecularColor, GBuffer.Roughness, NoV);

		WaterVolumeLightingOutput WaterLighting = EvaluateWaterVolumeLighting(
			MaterialParameters, PixelMaterialInputs, ResolvedView,
			DirectionalLightShadow, 
#if !MOBILE_SINGLELAYERWATER_FALLBACK
			MobileBasePass.SceneDepthCopyTexture, MobileBasePass.SceneDepthCopySampler,
			ResolvedView.BufferSizeAndInvSize.xy,
			ResolvedView.BufferSizeAndInvSize.zw,
			SceneColorCopyTexture, SceneColorCopySampler,
			SLWBufferBilinearUVMinMax.xy,
			SLWBufferBilinearUVMinMax.zw,
#endif
			GBuffer.Specular, MobileBasePass.DistortionParams,
			SunIlluminance, WaterDiffuseIndirectIlluminance, EnvBrdf,
			CameraIsUnderWater, WaterVisibility, EyeIndex,
			bSeparateMainDirLightLuminance, SeparatedWaterMainDirLightLuminance);

		// Accumulate luminance and occlude the background according to transmittance to view and mean transmittance to lights.
		LightAccumulator_AddSplit(DirectLighting, WaterLighting.Luminance, 0.0f, WaterLighting.Luminance, 1.0f, false);
#if !MOBILE_SINGLELAYERWATER_FALLBACK
		Opacity = 1.0f;
#else
		Opacity = 1.0 - ((1.0 - Opacity) * dot(WaterLighting.WaterToSceneToLightTransmittance, float3(1.0 / 3.0, 1.0 / 3.0, 1.0 / 3.0)));
#endif
#if MOBILE_EMULATION
		DiffuseColorForIndirect += WaterLighting.Luminance;
#endif
	}
#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#endif // !MOBILE_USE_GBUFFER || MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#endif // TEMPLATE_USES_SUBSTRATE


#if !SUBSTRATE_ENABLED || TEMPLATE_USES_LEGACYGBUFFERDATA || SUBSTRATE_INLINE_SINGLELAYERWATER

#if !MATERIAL_SHADINGMODEL_UNLIT
	half3 Color = DirectLighting.TotalLight;
#else
	half3 Color = 0.0f;
#endif

	half4 VertexFog = half4(0, 0, 0, 1);

#if USE_VERTEX_FOG
#if PACK_INTERPOLANTS
	VertexFog = PackedInterpolants[0];
#else
	VertexFog = BasePassInterpolants.VertexFog;
#endif
#endif
	// NEEDS_BASEPASS_PIXEL_FOGGING is not allowed on mobile for the sake of performance.
	
#if !MATERIAL_SHADINGMODEL_UNLIT && MOBILE_EMULATION
	Color = lerp(Color, DiffuseColorForIndirect, ResolvedView.UnlitViewmodeMask);
#endif

	half3 Emissive = GetMaterialEmissive(PixelMaterialInputs);

	Color += Emissive;

#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE
	half3 DualBlendSurfaceLuminancePostCoverage = 0.0f;
	half3 DualBlendSurfaceTransmittancePreCoverage = 1.0f;
	half DualBlendSurfaceCoverage = 1.0f;
#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && (!SUBSTRATE_ENABLED || TEMPLATE_USES_LEGACYGBUFFERDATA)
	{
		AccumulateThinTranslucentModel(
			DualBlendSurfaceLuminancePostCoverage,
			DualBlendSurfaceTransmittancePreCoverage,
			DualBlendSurfaceCoverage,
			MaterialParameters,
			GBuffer,
			DirectLighting.TotalLightDiffuse,
			DirectLighting.TotalLightSpecular,
			Emissive,
			Opacity);

		Color = 0;
		Opacity = 1.0f;
	}
#endif
#endif // MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE
	
	// MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT must come first because it also has MATERIALBLENDING_TRANSLUCENT defined
	#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE

		// Add fog luminance according to surfacecoverage and reduce surface luminance according to fog coverage.
		half3 AdjustedDualBlendAdd = DualBlendSurfaceCoverage * VertexFog.rgb + VertexFog.a * DualBlendSurfaceLuminancePostCoverage;
		// Fade the surface color transmittance out to 1 according to the surface coverage, and take into account the fog coverage to the surface.
		half3 AdjustedDualBlendMul = lerp(1.0f, VertexFog.a * DualBlendSurfaceTransmittancePreCoverage, DualBlendSurfaceCoverage);

		#if MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_SINGLE_SRC_BLENDING
			// In the fallback case, use standard alpha blending(grey scale transmittance)
			half AdjustedAlpha = saturate(1 - dot(AdjustedDualBlendMul, half3(1.0f, 1.0f, 1.0f) / 3.0f));
			OutColor = half4(AdjustedDualBlendAdd, AdjustedAlpha);
		#else
			OutColor = half4(AdjustedDualBlendAdd, 0.0);
			OutColor1 = half4(AdjustedDualBlendMul, 1.0);
		#endif
	// On mobile, water (an opaque material) is rendered as trnaslucent with forced premultiplied alpha blending (see MobileBasePass::SetTranslucentRenderState)
	#elif MATERIALBLENDING_ALPHACOMPOSITE || MATERIAL_SHADINGMODEL_SINGLELAYERWATER
		OutColor = half4(Color * VertexFog.a + VertexFog.rgb * Opacity, Opacity);
	#elif MATERIALBLENDING_ALPHAHOLDOUT
		// not implemented for holdout
		OutColor = half4(Color * VertexFog.a + VertexFog.rgb * Opacity, Opacity);
	#elif MATERIALBLENDING_TRANSLUCENT
		OutColor = half4(Color * VertexFog.a + VertexFog.rgb, Opacity);
	#elif MATERIALBLENDING_ADDITIVE
		OutColor = half4(Color * (VertexFog.a * Opacity.x), 0.0f);
	#elif MATERIALBLENDING_MODULATE
		half3 FoggedColor = lerp(half3(1, 1, 1), Color, VertexFog.aaa * VertexFog.aaa);
		OutColor = half4(FoggedColor, Opacity);
	#else
		OutColor.rgb = Color * VertexFog.a + VertexFog.rgb;

		#if !MATERIAL_USE_ALPHA_TO_COVERAGE
			// Planar reflections and scene captures use scene color alpha to keep track of where content has been rendered, for compositing into a different scene later
			OutColor.a = 0.0;
		#else
			half MaterialOpacityClip = GetMaterialOpacityMaskClipValue();
			float Mask = GetMaterialMask(PixelMaterialInputs) / (1.0 - MaterialOpacityClip);
			OutColor.a = uint((Mask + 0.25f) * 4.0f) / 4.0f;
		#endif
	#endif
#endif

#if HAS_SCENE_DEPTH_AUX_OUTPUT
	OutSceneDepthAux = SvPosition.z;
#endif

#if !MATERIALBLENDING_MODULATE
#if MATERIAL_IS_SKY
	// Dynamic capture exposure is 1 as of today.
	const float ViewPreExposure = View.RealTimeReflectionCapture > 0.0f ? View.RealTimeReflectionCapturePreExposure : View.PreExposure;
#else
	const float ViewPreExposure = View.PreExposure;
#endif
	OutColor.rgb *= ViewPreExposure;
#endif

#if MATERIAL_IS_SKY
	if (MobileBasePass.bApplyHalfResLocalFogToSkyMeshes)
	{
		// Apply the half resolution texture to the sky dome mesh when needed.
		const float2 ResolutionRatioCorrection = (ResolvedView.ViewSizeAndInvSize.xy * MobileBasePass.LFV.HalfResTextureSizeAndInvSize.zw * 0.5);
		const float2 HalfResSourceTexCoord = float2(SvPosition.xy - ResolvedView.ViewRectMin.xy) * ResolvedView.ViewSizeAndInvSize.zw * ResolutionRatioCorrection;
		float4 LFVContribution = Texture2DSampleLevel(MobileBasePass.HalfResLocalFogVolumeViewTexture, MobileBasePass.HalfResLocalFogVolumeViewSampler, HalfResSourceTexCoord, 0.0f);

		OutColor.rgb = OutColor.rgb * LFVContribution.aaa + LFVContribution.rgb;
	}
#endif

#if MATERIAL_IS_SKY || (!MATERIAL_SHADINGMODEL_UNLIT && !MOBILE_USE_GBUFFER)
	// clamp lit color to avoid overflows
	OutColor.rgb = SafeGetOutColor(OutColor.rgb);
#endif

#if USE_EDITOR_COMPOSITING && (MOBILE_EMULATION)
	// Editor primitive depth testing
	OutColor.a = 1.0;
	#if MATERIALBLENDING_MASKED
		// some material might have an opacity value
		OutColor.a = GetMaterialMaskInputRaw(PixelMaterialInputs);
	#endif
	clip(OutColor.a - GetMaterialOpacityMaskClipValue());
#else
	#if OUTPUT_GAMMA_SPACE
		OutColor.rgb = sqrt(OutColor.rgb);
	#endif
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK || LIGHTMAP_VT_ENABLED
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif

#if MOBILE_USE_GBUFFER && USE_GLES_FBF_DEFERRED
	OutProxy.rgb = OutColor.rgb;
#elif MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING
	OutProgrammableBlending = OutColor1 * FramebufferFetchColor0() + OutColor;
#endif
}
