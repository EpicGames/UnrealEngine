// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassVertexShader.usf: Base pass vertex shader used with forward shading
=============================================================================*/

#define IS_MOBILE_BASEPASS_VERTEX_SHADER 1

#include "Common.ush"

// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer
#define MobileSceneTextures MobileBasePass.SceneTextures

#include "/Engine/Generated/Material.ush"
#include "MobileBasePassCommon.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "HeightFogCommon.ush"

#if USE_VERTEX_FOG && PROJECT_SUPPORT_SKY_ATMOSPHERE
#include "SkyAtmosphereCommon.ush"
#endif

#if LOCAL_FOG_VOLUME_ON_TRANSLUCENT
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

struct FMobileShadingBasePassVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	FMobileBasePassInterpolantsVSToPS BasePassInterpolants;
	FStereoVSOutput StereoOutput;
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
};

#define FMobileShadingBasePassVSOutput FMobileShadingBasePassVSToPS
#define VertexFactoryGetInterpolants VertexFactoryGetInterpolantsVSToPS

/** Entry point for the base pass vertex shader. */
void Main(
	FVertexFactoryInput Input,
	out FMobileShadingBasePassVSOutput Output
	)
{
	StereoSetupVF(Input, Output.StereoOutput);
	const uint EyeIndex = GetEyeIndexFromVF(Input);

#if PACK_INTERPOLANTS
	float4 PackedInterps[NUM_VF_PACKED_INTERPOLANTS];
	UNROLL 
	for(int i = 0; i < NUM_VF_PACKED_INTERPOLANTS; ++i)
	{
		PackedInterps[i] = 0;
	}
#endif

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPositionExcludingWPO = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float4 WorldPosition = WorldPositionExcludingWPO;

	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);

	float3 WorldPositionOffset = GetMaterialWorldPositionOffset(VertexParameters);
	
	WorldPosition.xyz += WorldPositionOffset;

	ApplyMaterialFirstPersonTransform(VertexParameters, WorldPosition.xyz);

	float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
	Output.Position = INVARIANT(mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip));
	Output.BasePassInterpolants.PixelPosition = WorldPosition;

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
	Output.BasePassInterpolants.PixelPositionExcludingWPO = WorldPositionExcludingWPO.xyz;
#endif

#if USE_GLOBAL_CLIP_PLANE
	Output.BasePassInterpolants.OutClipDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPosition.xyz, 1));
#endif

#if USE_VERTEX_FOG

	half4 VertexFog = float4(0, 0, 0, 1);
	if (ResolvedView.RealTimeReflectionCapture == 0.0f)
	{
		// We only render fog on the sky dome per vertex if not rendering a real time reflection capture. A reflection capture have its own fog pass.

		float3 WorldPositionRelativeToCamera = WorldPosition.xyz - ResolvedView.TranslatedWorldCameraOrigin;
		if (MATERIAL_IS_SKY)
		{
			// If this is a IsSky material, then we evaluated the fog as if it was up to Far DeviceZ. This is to match deferred renderer SkyPass not writting depth.
			WorldPositionRelativeToCamera = normalize(WorldPosition.xyz - ResolvedView.TranslatedWorldCameraOrigin) * ConvertFromDeviceZ(max(1.0e-38, FarDepthValue));
		}

		VertexFog = CalculateHeightFog(WorldPositionRelativeToCamera, EyeIndex, ResolvedView); // WorldPosition is in fact TranslatedWorldPosition
	}

#if PROJECT_SUPPORT_SKY_ATMOSPHERE && MATERIAL_IS_SKY==0 // Do not apply aerial perpsective on sky materials
	if (ResolvedView.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f)
	{
		// Sample the aerial perspective (AP). It is also blended under the VertexFog parameter.
		VertexFog = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			Output.Position, (WorldPosition.xyz - ResolvedView.TranslatedWorldCameraOrigin) * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			ResolvedView.OneOverPreExposure, VertexFog);
	}
#endif

	// Apply local fog volume if enabled. Note: volumetric fog is not applied on mobile so it can result in visual discrepancy.
#if LOCAL_FOG_VOLUME_ON_TRANSLUCENT
	float4 VertexClipSpacePosition	= mul(float4(WorldPosition.xyz, 1), ResolvedView.TranslatedWorldToClip);
	float2 SvPosition				= (VertexClipSpacePosition.xy / VertexClipSpacePosition.w * float2(.5f, -.5f) + .5f) * ResolvedView.ViewSizeAndInvSize.xy;
	uint2 TilePos					= clamp(uint2(SvPosition.xy / float(LFVTilePixelSize)), uint2(0, 0), LFVTileDataResolution - 1);
	const bool bPlatformSupportsVolumetricFogOntranslucent = false; // Mobile does not do that so we try to compensate for it by evalauting LFV regularly on translucent.
	float4 LFVContribution			= GetLFVContribution(ResolvedView, TilePos, WorldPosition.xyz, bPlatformSupportsVolumetricFogOntranslucent);
	VertexFog						= float4(LFVContribution.rgb + VertexFog.rgb * LFVContribution.a, LFVContribution.a * VertexFog.a);
#endif // LOCAL_FOG_VOLUME_ON_TRANSLUCENT

#if PACK_INTERPOLANTS
	PackedInterps[0] = VertexFog;
#else
	Output.BasePassInterpolants.VertexFog = VertexFog;
#endif // PACK_INTERPOLANTS
#endif // USE_VERTEX_FOG

#if LANDSCAPE_BUG_WORKAROUND
	Output.BasePassInterpolants.DummyInterp = 0;
#endif



	Output.FactoryInterpolants = VertexFactoryGetInterpolants(Input, VFIntermediates, VertexParameters);

	Output.BasePassInterpolants.PixelPosition.w = Output.Position.w;

#if PACK_INTERPOLANTS
	VertexFactoryPackInterpolants(Output.FactoryInterpolants, PackedInterps);
#endif // PACK_INTERPOLANTS
}
