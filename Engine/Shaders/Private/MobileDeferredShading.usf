// Copyright Epic Games, Inc. All Rights Reserved.

#define MOBILE_DEFERRED_LIGHTING 1

#include "Common.ush"
#include "SHCommon.ush"

// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer 
#define MobileSceneTextures		MobileBasePass.SceneTextures
#define ForwardLightStruct		MobileBasePass.Forward
#define PlanarReflectionStruct	MobileBasePass.PlanarReflection
#define ReflectionStruct		MobileBasePass.ReflectionsParameters
#define PreIntegratedGF			ReflectionStruct.PreIntegratedGF
#define PreIntegratedGFSampler	ReflectionStruct.PreIntegratedGFSampler

#if MATERIAL_SHADER
#include "/Engine/Generated/Material.ush"
#endif

#ifndef SUPPORT_CONTACT_SHADOWS
#define SUPPORT_CONTACT_SHADOWS 0
#endif
#include "MobileLightingCommon.ush"
#if MATERIAL_SHADER
#include "LightFunctionCommon.ush"
#endif
#include "LightDataUniforms.ush"
#include "LightShaderParameters.ush"

#ifndef USE_HAIR_COMPLEX_TRANSMITTANCE
#define USE_HAIR_COMPLEX_TRANSMITTANCE 0
#endif
#include "HairStrands/HairStrandsEnvironmentLightingCommon.ush"

#if APPLY_SKY_SHADOWING
#include "SkyLightingDiffuseShared.ush"
#include "DistanceFieldAOShared.ush"
#endif

#ifndef RADIAL_LIGHT_TYPE
#define RADIAL_LIGHT_TYPE 0
#endif

#ifndef SUPPORT_RADIAL_LIGHT_SHADOW
#define SUPPORT_RADIAL_LIGHT_SHADOW 0
#endif 

#define SUPPORT_SPOTLIGHTS_SHADOW	SUPPORT_RADIAL_LIGHT_SHADOW && RADIAL_LIGHT_TYPE==LIGHT_TYPE_SPOT
#define SUPPORT_POINTLIGHTS_SHADOW	SUPPORT_RADIAL_LIGHT_SHADOW && RADIAL_LIGHT_TYPE==LIGHT_TYPE_POINT


float4x4 TranslatedWorldToLight;
float2 LightFunctionParameters2;

half ComputeLightFunctionMultiplier(float3 TranslatedWorldPosition)
{
#if USE_LIGHT_FUNCTION	
	float4 LightVector = mul(float4(TranslatedWorldPosition, 1.0), TranslatedWorldToLight);
	LightVector.xyz /= LightVector.w;

	half3 LightFunction = GetLightFunctionColor(LightVector.xyz, TranslatedWorldPosition);
	half GreyScale = dot(LightFunction, .3333f);
	// Calculate radial view distance for stable fading
	float ViewDistance = length(PrimaryView.TranslatedWorldCameraOrigin - TranslatedWorldPosition);
	half DistanceFadeAlpha = saturate((LightFunctionParameters2.x - ViewDistance) / (LightFunctionParameters2.x * .2f));
	// Fade to disabled based on LightFunctionFadeDistance
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, DistanceFadeAlpha);
	// Fade to disabled based on ShadowFadeFraction
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, LightFunctionParameters.y);
	return GreyScale; 
#else
	return 1.0;
#endif
}

half3 SkyLightDiffuseMobile(FGBufferData GBuffer, half3 DiffuseColor, half3 V, inout half IndirectIrradiance)
{
	half3 Lighting = 0;
	half3 SkyDiffuseLookUpNormal = GBuffer.WorldNormal;

	if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		half3 SubsurfaceLookup = GetSkySHDiffuseSimple(-GBuffer.WorldNormal) * View.SkyLightColor.rgb;
		half3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
		Lighting += SubsurfaceLookup * SubsurfaceColor;
	}

	if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		half3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
		// Add subsurface energy to diffuse
		DiffuseColor += SubsurfaceColor;
	}

	if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
	{
		const half3 N = GBuffer.WorldNormal;
		DiffuseColor = EvaluateEnvHair(GBuffer, V, N, SkyDiffuseLookUpNormal);
	}

	if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
	{
		half3 ClothFuzz = ExtractSubsurfaceColor(GBuffer);
		DiffuseColor += ClothFuzz * GBuffer.CustomData.a;
	}

	// Compute the preconvolved incoming lighting with the bent normal direction
	half3 DiffuseLookup = GetSkySHDiffuseSimple(SkyDiffuseLookUpNormal) * View.SkyLightColor.rgb;

	// And accumulate the lighting
	Lighting += DiffuseLookup  * DiffuseColor * GBuffer.GBufferAO;

#if ALLOW_STATIC_LIGHTING
	IndirectIrradiance += Luminance(DiffuseLookup);
#endif

	return Lighting;
}

void ReflectionEnvironmentSkyLighting(
	FGBufferData GBuffer,
	half4 SvPosition,
	half3 CameraVector,
	float3 TranslatedWorldPosition,
	half3 ReflectionVector,
	uint GridIndex,
	half GatheredAmbientOcclusion,
	inout FLightAccumulator DirectLighting)
{
	half IndirectIrradiance = GBuffer.IndirectIrradiance;

	// IBL
	half DiffuseAlpha = 
		AccumulateReflection(GBuffer
			, SvPosition
			, CameraVector
			, TranslatedWorldPosition
			, ReflectionVector
			, IndirectIrradiance
			, GridIndex
			, DirectLighting);

#if ENABLE_SKY_LIGHT
	BRANCH
	if (DiffuseAlpha > 0.0f)
	{
		half3 N = GBuffer.WorldNormal;
		half3 V = -CameraVector;
		half NoV = saturate(abs(dot(N, V)) + 1e-5);
#if APPLY_SKY_SHADOWING
		half3 BentNormal  = GBuffer.WorldNormal;
		const float2 BentNormalUV = (SvPosition.xy - View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
		BentNormal = UpsampleDFAO(BentNormalUV, GBuffer.Depth, GBuffer.WorldNormal);
#endif

		half3 DiffuseColor = GBuffer.DiffuseColor;
		half3 SpecularColor = GBuffer.SpecularColor;
		RemapClearCoatDiffuseAndSpecularColor(GBuffer, NoV, DiffuseColor, SpecularColor);
#if APPLY_SKY_SHADOWING
		//const float3 DiffuseColor = GBuffer.BaseColor; // not used
		const float2 BufferUV = SvPosition * View.BufferSizeAndInvSize.zw; // not used
		float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;
		// Sky lighting (already pre-exposed)
		half3 SkyLighting = SkyLightDiffuse(GBuffer, GatheredAmbientOcclusion, BufferUV, ScreenPosition, BentNormal, DiffuseColor);
#else
		half3 SkyLighting = SkyLightDiffuseMobile(GBuffer, DiffuseColor, V, IndirectIrradiance);
#endif

		SkyLighting *= DiffuseAlpha;

		const bool bNeedsSeparateSubsurfaceLightAccumulation = UseSubsurfaceProfile(GBuffer.ShadingModelID);
		LightAccumulator_Add(DirectLighting, SkyLighting, SkyLighting, 1.0f, bNeedsSeparateSubsurfaceLightAccumulation);
	}
#endif
}
  
void MobileDirectionalLightPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
#if MOBILE_MULTI_VIEW
	in nointerpolation uint ViewId : SV_ViewID,
#endif
	float4 SvPosition : SV_POSITION,
#if USE_GLES_FBF_DEFERRED
	out HALF4_TYPE OutProxyAdditive : SV_Target0,
	out HALF4_TYPE OutGBufferA : SV_Target1,
	out HALF4_TYPE OutGBufferB : SV_Target2,
	out HALF4_TYPE OutGBufferC : SV_Target3
#else
	out HALF4_TYPE OutColor : SV_Target0
#endif
)
{
#if MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(ViewId);
	const uint EyeIndex = ViewId;
#else
	ResolvedView = ResolveView();
	const uint EyeIndex = 0;
#endif

	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UVAndScreenPos.xy, UVAndScreenPos.zw);
	float2 ScreenPos = UVAndScreenPos.zw;
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPos, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	half3 CameraVector = normalize(TranslatedWorldPosition);
	half3 V = -CameraVector;
	half NoV = dot(GBuffer.WorldNormal, V);
	half3 ReflectionVector = GBuffer.WorldNormal * (NoV * 2.0) - V;

	FLightAccumulator DirectLighting = (FLightAccumulator)0;

	float4 ScreenPosition = SvPositionToScreenPosition(float4(SvPosition.xyz, GBuffer.Depth));

	half GatheredAmbientOcclusion = 1;
#if ENABLE_SKY_LIGHT && ENABLE_AMBIENT_OCCLUSION
	GatheredAmbientOcclusion = Texture2DSample(MobileBasePass.AmbientOcclusionTexture, MobileBasePass.AmbientOcclusionSampler, UVAndScreenPos.xy).r;
	#if ALLOW_STATIC_LIGHTING
		GatheredAmbientOcclusion = lerp(1.0, GatheredAmbientOcclusion, MobileBasePass.AmbientOcclusionStaticFraction);
	#endif
	GBuffer.GBufferAO *= GatheredAmbientOcclusion;
#endif

	// Directional light
	half4 DynamicShadowFactors = SampleScreenSpaceShadowMask(UVAndScreenPos.xy, EyeIndex);
	float DirectionalLightShadow = 1.0f;
	const float Dither = InterleavedGradientNoise(SvPosition.xy, View.StateFrameIndexMod8);
	AccumulateDirectionalLighting(GBuffer, TranslatedWorldPosition, CameraVector, ScreenPosition, DynamicShadowFactors, Dither, DirectionalLightShadow, DirectLighting);
	// LightFunction should only affect direct lighting result
	DirectLighting.TotalLight *= ComputeLightFunctionMultiplier(TranslatedWorldPosition);

	float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;
	uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), GBuffer.Depth, EyeIndex);
	// Local lights
#if ENABLE_CLUSTERED_LIGHTS
	{
		const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);
		half4 LocalLightDynamicShadowFactors = 1.0f;
		AccumulateLightGridLocalLighting(CulledLightGridHeader, GBuffer, TranslatedWorldPosition, CameraVector, EyeIndex, 0, LocalLightDynamicShadowFactors, DirectLighting);
	}
#endif

#if ENABLE_SKY_LIGHT
	// If we have a single directional light, apply relfection and sky contrubution here
	ReflectionEnvironmentSkyLighting(GBuffer, SvPosition, CameraVector, TranslatedWorldPosition, ReflectionVector, GridIndex, GatheredAmbientOcclusion, DirectLighting);
#endif

	half3 Color = DirectLighting.TotalLight;
	// MobileHDR applies PreExposure in tonemapper
	Color *= View.PreExposure;

	Color = SafeGetOutColor(Color);
#if USE_GLES_FBF_DEFERRED
	OutProxyAdditive.rgb = Color;
#else
	OutColor.rgb = Color;
	// Blend is enabled only for static skylights
	OutColor.a = GatheredAmbientOcclusion;
#endif
}

#if SUPPORT_SPOTLIGHTS_SHADOW
float4			SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale;
float4			SpotLightShadowmapMinMax;
float4x4		SpotLightShadowWorldToShadowMatrix;
Texture2D		LocalLightShadowTexture;
SamplerState	LocalLightShadowSampler;
float4			LocalLightShadowBufferSize;
#endif
uint ShadowedBits;

FDeferredLightData InitDeferredLightFromLightParameters(FLightShaderParameters In, uint InLightType)
{
	FDeferredLightData Out = (FDeferredLightData)0;
	Out.TranslatedWorldPosition	= In.TranslatedWorldPosition;
	Out.InvRadius				= In.InvRadius;
	Out.Color					= In.Color;
	Out.FalloffExponent			= In.FalloffExponent;
	Out.Direction				= In.Direction;
	Out.Tangent					= In.Tangent;
	Out.SpotAngles				= In.SpotAngles;
	Out.SpecularScale			= In.SpecularScale;
	Out.DiffuseScale			= In.DiffuseScale;
	Out.SourceRadius			= In.SourceRadius;
	Out.SoftSourceRadius		= In.SoftSourceRadius;
	Out.SourceLength			= In.SourceLength;
	
	Out.RectLightData.BarnCosAngle				= In.RectLightBarnCosAngle;
	Out.RectLightData.BarnLength				= In.RectLightBarnLength;
	Out.RectLightData.AtlasData.AtlasUVOffset	= In.RectLightAtlasUVOffset;
	Out.RectLightData.AtlasData.AtlasUVScale	= In.RectLightAtlasUVScale;
	Out.RectLightData.AtlasData.AtlasMaxLevel	= In.RectLightAtlasMaxLevel;

	Out.IESAtlasIndex			= In.IESAtlasIndex;
	Out.LightFunctionAtlasLightIndex = In.LightFunctionAtlasLightIndex;
	Out.bAffectsTranslucentLighting = In.bAffectsTranslucentLighting;
	
	Out.bInverseSquared			= In.FalloffExponent == 0;
	Out.bRadialLight			= true;
	Out.bSpotLight				= InLightType == LIGHT_TYPE_SPOT;
	Out.bRectLight				= InLightType == LIGHT_TYPE_RECT;
	Out.HairTransmittance		= InitHairTransmittanceData();
	Out.ShadowedBits						= ShadowedBits;

	MobileOverrideDeferredLightParameters(Out);

#if SUPPORT_SPOTLIGHTS_SHADOW
	Out.ShadowMapChannelMask	= 1.f;
#endif
	return Out;
}

void MobileRadialLightPS(
	float4 InScreenPosition : TEXCOORD0,
#if MOBILE_MULTI_VIEW
	in nointerpolation uint ViewId : SV_ViewID,
#endif
	float4 SVPos			: SV_POSITION,
#if USE_GLES_FBF_DEFERRED
	out HALF4_TYPE OutProxyAdditive : SV_Target0,
	out HALF4_TYPE OutGBufferA : SV_Target1,
	out HALF4_TYPE OutGBufferB : SV_Target2,
	out HALF4_TYPE OutGBufferC : SV_Target3
#else
	out HALF4_TYPE OutColor : SV_Target0
#endif
)
{
#if MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(ViewId);
	const uint EyeIndex = ViewId;
#else
	ResolvedView = ResolveView();
	const uint EyeIndex = 0;
#endif


	float2 ScreenUV = InScreenPosition.xy / InScreenPosition.w * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(ScreenUV, SVPos.xy);
	// With a perspective projection, the clip space position is NDC * Clip.w
	// With an orthographic projection, clip space is the same as NDC
	float2 ClipPosition = GetScreenPositionForProjectionType(InScreenPosition.xy / InScreenPosition.w, GBuffer.Depth);
	float3 TranslatedWorldPosition = mul(float4(ClipPosition, GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	half3 CameraVector = normalize(TranslatedWorldPosition);
			
	FLightAccumulator DirectLighting = (FLightAccumulator)0;
	FDeferredLightData LightData = InitDeferredLightFromLightParameters(GetRootLightShaderParameters(), RADIAL_LIGHT_TYPE);

#if !SIMPLE_LIGHT
	// Affect the light color by any other shadow/transmittance before the lighting is evaluated
	half Attenuation = ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);
	LightData.Color *= Attenuation;
#endif

	half Shadow = 1.0;
#if SUPPORT_SPOTLIGHTS_SHADOW
	float3 ToLight = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
	float3 LocalPosition = -ToLight;
	float DistanceSqr = dot(ToLight, ToLight);
	half3 L = ToLight * rsqrt(DistanceSqr);
	half3 N = GBuffer.WorldNormal;
	half NoL = saturate(dot(N, L));

	FPCFSamplerSettings Settings;
	Settings.ShadowDepthTexture = LocalLightShadowTexture;
	Settings.ShadowDepthTextureSampler = LocalLightShadowSampler;
	Settings.ShadowBufferSize = LocalLightShadowBufferSize;
	Settings.bSubsurface = false;
	Settings.bTreatMaxDepthUnshadowed = false;
	Settings.DensityMulConstant = 0;
	Settings.ProjectionDepthBiasParameters = 0;
	
	float SpotLightShadowSharpen = SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale.x;
	float SpotLightFadeFraction = SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale.y;
	float SpotLightReceiverDepthBias = SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale.z;
	float SpotLightSoftTransitionScale = SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale.w;

	float4 HomogeneousShadowPosition = mul(float4(LocalPosition, 1), SpotLightShadowWorldToShadowMatrix);
	float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;
	if (all(ShadowUVs >= SpotLightShadowmapMinMax.xy && ShadowUVs <= SpotLightShadowmapMinMax.zw))
	{
		// Clamp pixel depth in light space for shadowing opaque, because areas of the shadow depth buffer that weren't rendered to will have been cleared to 1
		// We want to force the shadow comparison to result in 'unshadowed' in that case, regardless of whether the pixel being shaded is in front or behind that plane
		// Invert ShadowZ as the shadow space has been changed (but not yet the filtering code)
		float ShadowZ = 1.0f - HomogeneousShadowPosition.z;
		float LightSpacePixelDepthForOpaque = min(ShadowZ, 0.99999f);
		Settings.SceneDepth = LightSpacePixelDepthForOpaque;
		Settings.TransitionScale = SpotLightSoftTransitionScale * lerp(SpotLightReceiverDepthBias, 1.0, NoL);

		Shadow = MobileShadowPCF(ShadowUVs, Settings);
		Shadow = saturate((Shadow - 0.5) * SpotLightShadowSharpen + 0.5f);
		Shadow = lerp(1.0f, Square(Shadow), SpotLightFadeFraction);
	}
#endif

#if SIMPLE_LIGHT
	FSimpleDeferredLightData SimpleLightData = (FSimpleDeferredLightData)0;
	SimpleLightData.TranslatedWorldPosition = LightData.TranslatedWorldPosition;
	SimpleLightData.InvRadius = LightData.InvRadius;
	SimpleLightData.Color = LightData.Color;
	SimpleLightData.FalloffExponent = LightData.FalloffExponent;
	SimpleLightData.bInverseSquared = LightData.bInverseSquared;
	half3 Color = GetSimpleDynamicLighting(TranslatedWorldPosition, CameraVector, GBuffer.WorldNormal, 1.0, 
		GBuffer.DiffuseColor * LightData.DiffuseScale, 
		GBuffer.SpecularColor * LightData.SpecularScale,
		GBuffer.Roughness, SimpleLightData);
#else
#if SUPPORT_POINTLIGHTS_SHADOW
	half4 LightAttenuation = SampleScreenSpaceShadowMask(ScreenUV, EyeIndex);
#else
	half4 LightAttenuation = half4(1, 1, Shadow, Shadow);
#endif

	float SurfaceShadow = 0;
	const float Dither = InterleavedGradientNoise(SVPos.xy, View.StateFrameIndexMod8);
	DirectLighting = AccumulateDynamicLighting(TranslatedWorldPosition, CameraVector, GBuffer, 1, LightData, LightAttenuation, Dither, uint2(0, 0), SurfaceShadow);
	half3 Color = DirectLighting.TotalLight * ComputeLightFunctionMultiplier(TranslatedWorldPosition);
#endif

	// MobileHDR applies PreExposure in tonemapper
	Color *= View.PreExposure;
	Color = SafeGetOutColor(Color);

#if USE_GLES_FBF_DEFERRED
	OutProxyAdditive.rgb = Color;
#else
	OutColor.rgb = Color;
	OutColor.a = 1;
#endif
}

void MobileReflectionEnvironmentSkyLightingPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
	, float4 SvPosition : SV_POSITION
#if USE_GLES_FBF_DEFERRED
	, out HALF4_TYPE OutProxyAdditive : SV_Target0
	, out HALF4_TYPE OutGBufferA : SV_Target1
	, out HALF4_TYPE OutGBufferB : SV_Target2
	, out HALF4_TYPE OutGBufferC : SV_Target3
#else
	, out HALF4_TYPE OutColor : SV_Target0
#endif
)
{
	ResolvedView = ResolveView();
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UVAndScreenPos.xy, UVAndScreenPos.zw);
	float2 ScreenPos = UVAndScreenPos.zw;
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPos, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	half3 CameraVector = normalize(TranslatedWorldPosition);
	half3 V = -CameraVector;
	half NoV = max(0, dot(GBuffer.WorldNormal, V));
	half3 ReflectionVector = GBuffer.WorldNormal * (NoV * 2.0) - V;

	const uint EyeIndex = 0;
	float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;
	uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), GBuffer.Depth, EyeIndex);

	half GatheredAmbientOcclusion = 1;
#if (ENABLE_SKY_LIGHT || DIFFUSE_FROM_CAPTURE) && ENABLE_AMBIENT_OCCLUSION
	GatheredAmbientOcclusion = Texture2DSample(MobileBasePass.AmbientOcclusionTexture, MobileBasePass.AmbientOcclusionSampler, UVAndScreenPos.xy).r;
	#if ALLOW_STATIC_LIGHTING
		GatheredAmbientOcclusion = lerp(1.0, GatheredAmbientOcclusion, MobileBasePass.AmbientOcclusionStaticFraction);
	#endif
	GBuffer.GBufferAO *= GatheredAmbientOcclusion;
#endif
	
	FLightAccumulator DirectLighting = (FLightAccumulator)0;
	ReflectionEnvironmentSkyLighting(GBuffer, SvPosition, CameraVector, TranslatedWorldPosition, ReflectionVector, GridIndex, GatheredAmbientOcclusion, DirectLighting);
	half3 ReflectionAndSky = DirectLighting.TotalLight;

	ReflectionAndSky *= View.PreExposure;

#if USE_GLES_FBF_DEFERRED
	OutProxyAdditive.rgb = ReflectionAndSky.rgb;
#else
	OutColor.rgb = ReflectionAndSky.rgb;
	// Blend is enabled only for static skylights
	OutColor.a = GatheredAmbientOcclusion;
#endif
}

// Write of sky light pixels to sky needs to not include SkylightScale, which is used to adjust indirect
// skylight on surfaces, not the sky itself.  So we use a separate parameter.
float4 SkylightUnscaledColor;

// Writes sky light color to sky pixels (assumes stencil is set up to mask sky pixels)
void MobileSkyLightToSkyPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
#if USE_GLES_FBF_DEFERRED
	out HALF4_TYPE OutProxyAdditive : SV_Target0,
	out HALF4_TYPE OutGBufferA : SV_Target1,
	out HALF4_TYPE OutGBufferB : SV_Target2,
	out HALF4_TYPE OutGBufferC : SV_Target3
#else
	out HALF4_TYPE OutColor : SV_Target0
#endif
	)
{
	FGBufferData GBuffer = MobileFetchAndDecodeGBuffer(UVAndScreenPos.xy, UVAndScreenPos.zw);
	float2 ScreenPos = UVAndScreenPos.zw;
	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPos, GBuffer.Depth), GBuffer.Depth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	half3 Color = TextureCubeSampleLevel(ReflectionStruct.SkyLightCubemap, SharedSkyLightCubemapSampler, TranslatedWorldPosition, 0).rgb * SkylightUnscaledColor.rgb;

#if USE_GLES_FBF_DEFERRED
	OutProxyAdditive.rgb = Color;
#else
	OutColor.rgb = Color;
	OutColor.a = 1;
#endif
}
