// Copyright Epic Games, Inc. All Rights Reserved.
#include "Common.ush"

#if USE_GLES_FBF_DEFERRED
#include "/Engine/Public/Platform/GL/GLSubpassSupport.ush"
#endif

void MobileDeferredCopyPLSPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0, 
	float4 SvPosition : SV_POSITION, 
	out half4 OutColor : SV_Target0)
{
#if USE_GLES_FBF_DEFERRED
	OutColor = half4(GLSubpassFetch0().rgb, 1.0);
#else
	OutColor = 1.0f;  
#endif
}

void MobileDeferredCopyDepthPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0, 
	float4 SvPosition : SV_POSITION, 
	out float OutSceneDepthAux : SV_Target0)
{
#if USE_GLES_FBF_DEFERRED
	OutSceneDepthAux = DepthbufferFetchES2();
#else
	OutSceneDepthAux = 1.0f;  
#endif
}
