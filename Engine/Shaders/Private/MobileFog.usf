// Copyright Epic Games, Inc. All Rights Reserved.
#if SHADING_PATH_MOBILE
#include "Common.ush"

// Reroute MobileSceneTextures uniform buffer references to the base pass uniform buffer 
#define MobileSceneTextures MobileBasePass.SceneTextures
#define FogStruct MobileBasePass.Fog

#include "SceneTexturesCommon.ush"
#include "HeightFogCommon.ush"
#include "SkyAtmosphereCommon.ush"

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME > 0
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME == 2
Texture2D<float4> HalfResLocalFogVolumeViewSRV;
SamplerState HalfResLocalFogVolumeViewSRVSampler;
Texture2D<float4>  HalfResLocalFogVolumeDepthSRV;
SamplerState HalfResLocalFogVolumeDepthSRVSampler;
#endif

float StartDepthZ;

void MobileFogVS(
	in FStereoVSInput StereoInput,
	out FStereoVSOutput StereoOutput,
	in float2 InPosition : ATTRIBUTE0,
	out float2 OutTexCoord : TEXCOORD0,
	out float3 OutScreenVector : TEXCOORD1,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);

	// screenspace position from vb
	OutPosition = float4(InPosition,StartDepthZ,1);
	// texture coord from vb
	OutTexCoord = InPosition * ResolvedView.ScreenPositionScaleBias.xy + ResolvedView.ScreenPositionScaleBias.wz;
	// deproject to world space
	OutScreenVector = ScreenVectorFromScreenRect(float4(InPosition,1,0), ResolvedView);
}

void MobileFogPS(
	in FStereoPSInput StereoInput,
	float2 TexCoord : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	StereoSetupPS(StereoInput);

	half4 FinalFog = half4(0,0,0,1);
	
	float DeviceZ = LookupDeviceZ(TexCoord, GetEyeIndex(StereoInput));
	float SceneDepth = ConvertFromDeviceZ(max(1.0e-10, DeviceZ));
	float3 WorldPositionRelativeToCamera = ScreenVector.xyz * SceneDepth;
	float3 TranslatedWorldPosition = WorldPositionRelativeToCamera + GetTranslatedWorldCameraPosFromView(SVPos.xy);

// at least one of these is active
#if PERMUTATION_SUPPORT_HEIGHT_FOG
	FinalFog = CalculateHeightFog(WorldPositionRelativeToCamera, GetEyeIndex(StereoInput), ResolvedView);
#endif

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME == 1

	// Evaluate the local fog volumes in line when there is volumetric fog.
	uint2 TilePos = uint2(SVPos.xy - ResolvedView.ViewRectMin.xy) / LFVTilePixelSize.xx;
	float4 LFVContribution = GetLFVContribution(ResolvedView, TilePos, WorldPositionRelativeToCamera);
	FinalFog = float4(LFVContribution.rgb + FinalFog.rgb * LFVContribution.a, LFVContribution.a * FinalFog.a);

#elif PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME == 2

	// Upsample the half resolution local fog volumes previously traced in a half resolution texture.
	const int2 FullResolutionToVolumetricBufferResolutionScale = uint2(2, 2);

	int2 PixelPos = SVPos.xy - ResolvedView.ViewRectMin.xy;
	int2 VolumeCoordUInt = PixelPos / int(FullResolutionToVolumetricBufferResolutionScale.y);
	int OffsetX = (VolumeCoordUInt.x * int(FullResolutionToVolumetricBufferResolutionScale.y)) == PixelPos.x ? -1 : 1;
	int OffsetY = (VolumeCoordUInt.y * int(FullResolutionToVolumetricBufferResolutionScale.y)) == PixelPos.y ? -1 : 1;

	uint2 VolumeCoord0 = max(0, int2(VolumeCoordUInt) + int2(0, 0));
	uint2 VolumeCoord1 = max(0, int2(VolumeCoordUInt) + int2(OffsetX, 0));
	uint2 VolumeCoord2 = max(0, int2(VolumeCoordUInt) + int2(OffsetX, OffsetY));
	uint2 VolumeCoord3 = max(0, int2(VolumeCoordUInt) + int2(0, OffsetY));

	float VolumeFrontDepth0 = HalfResLocalFogVolumeDepthSRV.Load(uint3(VolumeCoord0, 0)).r * METER_TO_CENTIMETER;
	float VolumeFrontDepth1 = HalfResLocalFogVolumeDepthSRV.Load(uint3(VolumeCoord1, 0)).r * METER_TO_CENTIMETER;
	float VolumeFrontDepth2 = HalfResLocalFogVolumeDepthSRV.Load(uint3(VolumeCoord2, 0)).r * METER_TO_CENTIMETER;
	float VolumeFrontDepth3 = HalfResLocalFogVolumeDepthSRV.Load(uint3(VolumeCoord3, 0)).r * METER_TO_CENTIMETER;

	float MaxFrontDepth4Diff = max(max(VolumeFrontDepth0, VolumeFrontDepth1), max(VolumeFrontDepth2, VolumeFrontDepth3));
	float UpsampleSceneDepth = min(SceneDepth, MaxFrontDepth4Diff);	// needed because FP16 is limited in distance and that fixes upsampling artefacts when depth is super large..

	float4 Depth4 = float4(VolumeFrontDepth0, VolumeFrontDepth1, VolumeFrontDepth2, VolumeFrontDepth3);
	float4 Depth4Diff = UpsampleSceneDepth - Depth4;
	Depth4Diff = abs(Depth4Diff);
	float MaxDepth4Diff = max(max(Depth4Diff.x, Depth4Diff.y), max(Depth4Diff.z, Depth4Diff.w));

	const float WeightMultiplier = 10.0f;
	const float ThresholdToBilinear = 30.0f; // in centimeters
	float4 LFVContribution;
	if (MaxDepth4Diff > ThresholdToBilinear)
	{
		float MaxDepth4Diff = max(max(Depth4Diff.x, Depth4Diff.y), max(Depth4Diff.z, Depth4Diff.w));
		// Depth discontinuities edges
		float4 weights = 1.0f / (Depth4Diff * WeightMultiplier + 1.0f);
		const float weightsSum = dot(weights, float4(1.0f, 1.0f, 1.0f, 1.0f));
		weights /= weightsSum;

		float4 VolumeRGBT0 = HalfResLocalFogVolumeViewSRV.Load(uint3(VolumeCoord0, 0));
		float4 VolumeRGBT1 = HalfResLocalFogVolumeViewSRV.Load(uint3(VolumeCoord1, 0));
		float4 VolumeRGBT2 = HalfResLocalFogVolumeViewSRV.Load(uint3(VolumeCoord2, 0));
		float4 VolumeRGBT3 = HalfResLocalFogVolumeViewSRV.Load(uint3(VolumeCoord3, 0));

		LFVContribution = weights.x * VolumeRGBT0 + weights.y * VolumeRGBT1 + weights.z * VolumeRGBT2 + weights.w * VolumeRGBT3;
	}
	else
	{
		// This is needed when half res is not exactly a multiple of 2, for the bilinear filtering to work nicely.
		const float2 ResolutionRatioCorrection = (ResolvedView.ViewSizeAndInvSize.xy * LFVHalfResTextureSizeAndInvSize.zw * 0.5);
		const float2 HalfResSourceTexCoord = float2(SVPos.xy - ResolvedView.ViewRectMin.xy) * ResolvedView.ViewSizeAndInvSize.zw * ResolutionRatioCorrection;
		LFVContribution = Texture2DSampleLevel(HalfResLocalFogVolumeViewSRV, HalfResLocalFogVolumeViewSRVSampler, HalfResSourceTexCoord, 0.0f); // Simple bilinear filtering for increased performance
	}

	// HalfResLocalFogVolumeViewSRV contains pre exposed color, so remove it for composition.
	LFVContribution.rgb *= ResolvedView.OneOverPreExposure;

	// Combine with the curren fog
	FinalFog = float4(LFVContribution.rgb + FinalFog.rgb * LFVContribution.a, LFVContribution.a * FinalFog.a);

#endif // PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME

#if PERMUTATION_SUPPORT_VOLUMETRIC_FOG
	if (FogStruct.ApplyVolumetricFog > 0) 
	{
		float3 VolumeUV = ComputeVolumeUVFromTranslatedPos(TranslatedWorldPosition, ResolvedView.TranslatedWorldToClip, ResolvedView);
		FinalFog = CombineVolumetricFog(FinalFog, VolumeUV, GetEyeIndex(StereoInput), SceneDepth, ResolvedView);
	}
#endif

#if PERMUTATION_SUPPORT_AERIAL_PERSPECTIVE
	if (ResolvedView.SkyAtmosphereApplyCameraAerialPerspectiveVolume > 0.0f && (DeviceZ != FarDepthValue))
	{
		float4 NDCPosition = mul(float4(TranslatedWorldPosition, 1.0f), ResolvedView.TranslatedWorldToClip);

		// Sample the aerial perspective (AP). It is also blended under the VertexFog parameter.
		FinalFog = GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
			ResolvedView.RealTimeReflectionCapture, 
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
			NDCPosition, 
			(WorldPositionRelativeToCamera.xyz - ResolvedView.TranslatedWorldCameraOrigin) * CM_TO_SKY_UNIT,
			View.CameraAerialPerspectiveVolume, 
			View.CameraAerialPerspectiveVolumeSampler,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,
			ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,
			ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
			ResolvedView.OneOverPreExposure, 
			FinalFog);
	}
#endif// PERMUTATION_SUPPORT_AERIAL_PERSPECTIVE

	FinalFog.rgb *= ResolvedView.PreExposure;
	OutColor = RETURN_COLOR(half4(FinalFog.rgb, FinalFog.w)); 
}
#endif //SHADING_PATH_MOBILE