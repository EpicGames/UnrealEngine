/*
 * Copyright (C) 2017 The Android Open Source Project
 *
 * Licensed under the Apache License, Version 2.0 (the "License");
 * you may not use this file except in compliance with the License.
 * You may obtain a copy of the License at
 *
 *      http://www.apache.org/licenses/LICENSE-2.0
 *
 * Unless required by applicable law or agreed to in writing, software
 * distributed under the License is distributed on an "AS IS" BASIS,
 * WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
 * See the License for the specific language governing permissions and
 * limitations under the License.
 */

#include "/Engine/Public/Platform.ush"

#define MEDIUMP_FLT_MAX		65504.0
#define MEDIUMP_FLT_MIN		0.00006103515625

// Taken from https://gist.github.com/romainguy/a2e9208f14cae37c579448be99f78f25
// Modified by Epic Games, Inc. To account for premultiplied light color and code style rules.

half D_GGX_Mobile(half Roughness, float NoH)
{
    // Walter et al. 2007, "Microfacet Models for Refraction through Rough Surfaces"
	float OneMinusNoHSqr = 1.0 - NoH * NoH; 
	half a = Roughness * Roughness;
	half n = NoH * a;
	half p = a / (OneMinusNoHSqr + n * n);
	half d = (1.0 / PI) * p * p;
	// clamp to avoid overlfow in a bright env
	return min(d, 2048.0);
}