// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef SUPPORT_CONTACT_SHADOWS
#define SUPPORT_CONTACT_SHADOWS 0
#endif

#include "ShadingModels.ush"
#include "ReflectionEnvironmentShared.ush"
#include "PlanarReflectionShared.ush"
#include "LightGridCommon.ush"
#include "ShadowFilteringCommon.ush"
#include "ClearCoatCommon.ush"
#include "DeferredLightingCommon.ush"
#include "SHCommon.ush"

#if MOBILE_SSR_ENABLED && !(MOBILE_QL_FORCE_FULLY_ROUGH || MATERIAL_FULLY_ROUGH)
	#if MOBILE_DEFERRED_SHADING
		#if IS_MOBILE_BASE_PASS
			#define MOBILE_USE_SSR (MATERIAL_SSR || (MATERIAL_SHADINGMODEL_SINGLELAYERWATER && !MOBILE_SINGLELAYERWATER_FALLBACK))
		#else
			#define MOBILE_USE_SSR 1
		#endif
	#else
		#define MOBILE_USE_SSR (MATERIAL_SSR || !MATERIALBLENDING_ANY_TRANSLUCENT)
	#endif
	#define SSRData	MobileBasePass.SSRParams
#else 
	#define MOBILE_USE_SSR 0
#endif

#if MOBILE_USE_SSR
#include "MobileSSR.ush"
#endif

#ifndef TRANSLUCENCY_NON_DIRECTIONAL
#define TRANSLUCENCY_NON_DIRECTIONAL 0
#endif

#ifndef MOBILE_USE_CSM_BRANCH
#define MOBILE_USE_CSM_BRANCH 0
#endif

#ifndef LQ_TEXTURE_LIGHTMAP
#define LQ_TEXTURE_LIGHTMAP 0
#endif

#ifndef ENABLE_PLANAR_REFLECTION
#define ENABLE_PLANAR_REFLECTION 0
#endif 

#ifndef MOBILE_SHADINGMODEL_SUPPORT
#define MOBILE_SHADINGMODEL_SUPPORT 0
#endif

#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES 1
#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 0 // Adds additional sampler
#define REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES 1
#define REFLECTION_COMPOSITE_HAS_BOX_CAPTURES 1
#include "ReflectionEnvironmentComposite.ush"

#if SUBSTRATE_ENABLED
#include "Substrate/Substrate.ush"
#endif

half3 SafeGetOutColor(half3 OutColor)
{
	// Sky materials can result in high luminance values, e.g. the sun disk. 
	// The PreExposure could double the OutColor
	// This is so we make sure to at least stay within the boundaries of fp10 and not cause NaN on some platforms.
	// We also half that range to also make sure we have room for other additive elements such as bloom, clouds or particle visual effects.
	OutColor = min(OutColor, Max111110BitsFloat3 * 0.5f);
	return OutColor;
}

FLightAccumulator LightAccumulator_Add(FLightAccumulator A, FLightAccumulator B)
{
	FLightAccumulator Sum = (FLightAccumulator)0;
	Sum.TotalLight = A.TotalLight + B.TotalLight;
	Sum.ScatterableLightLuma = A.ScatterableLightLuma + B.ScatterableLightLuma;
	Sum.ScatterableLight = A.ScatterableLight + B.ScatterableLight;
	Sum.EstimatedCost = A.EstimatedCost + B.EstimatedCost;
	Sum.TotalLightDiffuse = A.TotalLightDiffuse + B.TotalLightDiffuse;
	Sum.TotalLightSpecular = A.TotalLightSpecular + B.TotalLightSpecular;
	return Sum;
}

/*------------------------------------------------------------------------------
	Mobile Shadow.
------------------------------------------------------------------------------*/

half ApplyPrecomputedShadowMask(half ShadowMap, half Shadow)
{
#if ALLOW_STATIC_LIGHTING
	return min(ShadowMap, Shadow);
#else
	return ShadowMap;
#endif
}

#if MOBILE_USE_CSM_BRANCH
	uint UseCSM;
#endif

bool IsCSMEnabled()
{
#if MOBILE_USE_CSM_BRANCH
	return UseCSM != 0u;
#else
	return true;
#endif
}

half MobileShadowPCF(float2 ShadowUVs, FPCFSamplerSettings Settings)
{
#if MOBILE_SHADOW_QUALITY == 0
	half ShadowMap = ManualNoFiltering(ShadowUVs, Settings);
#elif MOBILE_SHADOW_QUALITY == 1
	half ShadowMap = Manual1x1PCF(ShadowUVs, Settings);
#elif MOBILE_SHADOW_QUALITY == 2
	half ShadowMap = Manual3x3PCF(ShadowUVs, Settings);
#elif MOBILE_SHADOW_QUALITY == 3
	half ShadowMap = Manual5x5PCF(ShadowUVs, Settings);
#else
	#error Unsupported MOBILE_SHADOW_QUALITY value.
#endif

return ShadowMap;
}

// Add fading CSM plane:
#define FADE_CSM 1

#ifndef ENABLE_MOBILE_CSM
#define ENABLE_MOBILE_CSM 1
#endif

#ifndef MAX_MOBILE_SHADOWCASCADES
#define MAX_MOBILE_SHADOWCASCADES 4u
#endif 

struct FDirectionalShadowSettings
{
	bool bSubsurface;
	float DensityMulConstant;
};

FDirectionalShadowSettings GetDefaultDirectionalShadowSettings()
{
	FDirectionalShadowSettings DirectionalShadowSettings;
	DirectionalShadowSettings.bSubsurface = false;
	DirectionalShadowSettings.DensityMulConstant = 0.0f;
	return DirectionalShadowSettings;
}

half MobileDirectionalLightCSM(float2 ScreenPosition, float SceneDepth, inout float ShadowPositionZ, FDirectionalShadowSettings DirectionalShadowSettings)
{
	half ShadowMap = 1;
#if ENABLE_MOBILE_CSM
	ShadowPositionZ = 0;
	FPCFSamplerSettings Settings;
	Settings.ShadowDepthTexture = MobileDirectionalLight.DirectionalLightShadowTexture;
	Settings.ShadowDepthTextureSampler = MobileDirectionalLight.DirectionalLightShadowSampler;
	Settings.TransitionScale = MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.w;
	Settings.ShadowBufferSize = MobileDirectionalLight.DirectionalLightShadowSize;
	Settings.bSubsurface = DirectionalShadowSettings.bSubsurface;
	Settings.bTreatMaxDepthUnshadowed = false;
	Settings.ProjectionDepthBiasParameters = 0;

	float4 Count = float4(SceneDepth.xxxx >= MobileDirectionalLight.DirectionalLightShadowDistances);
	uint CascadeIndex = uint(Count.x + Count.y + Count.z + Count.w);
	
	if (CascadeIndex < MobileDirectionalLight.DirectionalLightNumCascades)
	{
		float4 ShadowPosition = float4(0, 0, 0, 0);
	#if MOBILE_MULTI_VIEW
		ShadowPosition = mul(float4(ScreenPosition.x, ScreenPosition.y, SceneDepth, 1), ResolvedView.MobileMultiviewShadowTransform);
		ShadowPosition = mul(ShadowPosition, MobileDirectionalLight.DirectionalLightScreenToShadow[CascadeIndex]);
	#else
		ShadowPosition = mul(float4(ScreenPosition.x, ScreenPosition.y, SceneDepth, 1), MobileDirectionalLight.DirectionalLightScreenToShadow[CascadeIndex]);
	#endif
		ShadowPositionZ = ShadowPosition.z;

		// Process CSM only when ShadowPosition is valid.
		if (ShadowPosition.z > 0)
		{
			Settings.DensityMulConstant = DirectionalShadowSettings.DensityMulConstant * MobileDirectionalLight.DirectionalLightShadowDepthRange[CascadeIndex];

			// Clamp pixel depth in light space for shadowing opaque, because areas of the shadow depth buffer that weren't rendered to will have been cleared to 1
			// We want to force the shadow comparison to result in 'unshadowed' in that case, regardless of whether the pixel being shaded is in front or behind that plane

			// Invert ShadowZ as the shadow space has been changed (but not yet the filtering code)
			float ShadowZ = 1.0f - ShadowPosition.z;
			float LightSpacePixelDepthForOpaque = min(ShadowZ, 0.99999f);
			Settings.SceneDepth = LightSpacePixelDepthForOpaque;

			ShadowMap = MobileShadowPCF(ShadowPosition.xy, Settings);

		#if FADE_CSM
			float Fade = saturate(SceneDepth * MobileDirectionalLight.DirectionalLightDistanceFadeMADAndSpecularScale.x + MobileDirectionalLight.DirectionalLightDistanceFadeMADAndSpecularScale.y);
			// lerp out shadow based on fade params.
			ShadowMap = lerp(ShadowMap, 1.0, Fade * Fade);
		#endif
		}
	}
#endif //ENABLE_MOBILE_CSM
	return ShadowMap;
}

/*------------------------------------------------------------------------------
	Mobile Translucency Light.
------------------------------------------------------------------------------*/

struct FTranslucencyLightingVector
{
	float4 AmbientLightingVector;
	float3 DirectionalLightingVector;
};

FTranslucencyLightingVector GetTranslucencyLightingVector(FDeferredLightData LightData, float3 TranslatedWorldPosition)
{
	FTranslucencyLightingVector Out;

	float Attenuation = 1.0f;
	float3 L = LightData.Direction;	// Already normalized
	float3 NormalizedLightVector = L;
	if (LightData.bRadialLight)
	{
		Attenuation = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, L, NormalizedLightVector);
	}

	float3 Lighting = LightData.Color / PI * Attenuation;
	FTwoBandSHVectorRGB SHLighting = MulSH(SHBasisFunction(NormalizedLightVector), Lighting);

	Out.AmbientLightingVector = float4(SHLighting.R.V.x, SHLighting.G.V.x, SHLighting.B.V.x, 1.0f);

	float3 LuminanceWeights = LuminanceFactors();
	float3 Coefficient0 = float3(SHLighting.R.V.y, SHLighting.G.V.y, SHLighting.B.V.y);
	float3 Coefficient1 = float3(SHLighting.R.V.z, SHLighting.G.V.z, SHLighting.B.V.z);
	float3 Coefficient2 = float3(SHLighting.R.V.w, SHLighting.G.V.w, SHLighting.B.V.w);
	Out.DirectionalLightingVector = float3(dot(Coefficient0, LuminanceWeights), dot(Coefficient1, LuminanceWeights), dot(Coefficient2, LuminanceWeights));

	return Out;
}

#if TRANSLUCENCY_SH_LIGHTING

FLightAccumulator GetTranslucencySHLighting(FGBufferData GBuffer, FDeferredLightData LightData, float3 TranslatedWorldPosition)
{
	float4 VolumeLighting;
	float3 InterpolatedLighting = 0;

	FTranslucencyLightingVector LightingVector = GetTranslucencyLightingVector(LightData, TranslatedWorldPosition);

#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME

	GetVolumeLightingDirectional(LightingVector.AmbientLightingVector, LightingVector.DirectionalLightingVector, GBuffer.WorldNormal, GBuffer.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), InterpolatedLighting, VolumeLighting);

#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL

	GetVolumeLightingNonDirectional(LightingVector.AmbientLightingVector, GBuffer.DiffuseColor, InterpolatedLighting, VolumeLighting);

#endif

	FLightAccumulator LightAccumulator = (FLightAccumulator)0;
	LightAccumulator_AddSplit(LightAccumulator, InterpolatedLighting, 0.0f, InterpolatedLighting, 1.0f, false);
	return LightAccumulator;
}

#endif

/*------------------------------------------------------------------------------
	Mobile Directional Light.
------------------------------------------------------------------------------*/
half4 SampleScreenSpaceShadowMask(float2 BufferUV, uint EyeIndex)
{
#if ENABLE_SHADOWMASKTEXTURE 
	#if MOBILE_MULTI_VIEW
		return Texture2DArraySample(MobileBasePass.ScreenSpaceShadowMaskTextureArray, MobileBasePass.ScreenSpaceShadowMaskSampler, float3(BufferUV, EyeIndex));
	#else
		return Texture2DSample(MobileBasePass.ScreenSpaceShadowMaskTexture, MobileBasePass.ScreenSpaceShadowMaskSampler, BufferUV);
	#endif
#endif
	return 1.0;
}

FDeferredLightData GetDirectionalLightData(float4 ScreenPosition, inout half OutDynamicShadowing, FDirectionalShadowSettings DirectionalShadowSettings)
{
	float  ShadowPositionZ = 0;
	half ShadowMap = MobileDirectionalLightCSM(ScreenPosition.xy, ScreenPosition.w, ShadowPositionZ, DirectionalShadowSettings);
	OutDynamicShadowing = 1.0f;

#if !MOBILE_DEFERRED_SHADING && DIRECTIONAL_LIGHT_CSM
	// Cascaded Shadow Map
	if (IsCSMEnabled())
	{		
		OutDynamicShadowing = ShadowMap;
	}
#elif MOBILE_DEFERRED_SHADING
	OutDynamicShadowing = ShadowMap;
#endif

	FDeferredLightData LightData = (FDeferredLightData)0;
	LightData.Color = MobileDirectionalLight.DirectionalLightColor.rgb;
	LightData.FalloffExponent = 0;
	LightData.Direction = MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.xyz;
	LightData.bRadialLight = false;
	LightData.SpecularScale = MobileDirectionalLight.DirectionalLightDistanceFadeMADAndSpecularScale.z;
	LightData.DiffuseScale = MobileDirectionalLight.DirectionalLightDistanceFadeMADAndSpecularScale.w;

	LightData.ShadowedBits = MobileDirectionalLight.ShadowedBits;
	LightData.HairTransmittance = InitHairTransmittanceData();
	LightData.ShadowMapChannelMask = UnpackShadowMapChannelMask(MobileDirectionalLight.DirectionalLightShadowMapChannelMask);

	return LightData;
}

void AccumulateDirectionalLighting(FGBufferData GBuffer, float3 TranslatedWorldPosition, half3 CameraVector, float4 ScreenPosition, in half4 DynamicShadowFactors, in float Dither, inout float OutDirectionalLightShadow, inout FLightAccumulator DirectLighting)
{
	FDirectionalShadowSettings DirectionalShadowSettings = GetDefaultDirectionalShadowSettings();
#if MOBILE_SHADINGMODEL_SUPPORT && SUBSTRATE_ENABLED
	if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
	{
		DirectionalShadowSettings.bSubsurface = true;

		float ToonProfileId = 0.0f;
		float2 PatternUVs = 0.0f;
		UnpackToonCustomData(PackRGBA8(GBuffer.CustomData), PatternUVs, ToonProfileId);

		// Density is extinction as 1/cm (not 1/m) to match with the transmittance shadow evaluation from ManualPCF below.
		FToonProfileShadowSettings ToonProfileShadowSettings = GetToonProfileShadowSettings(ToonProfileId);
		DirectionalShadowSettings.DensityMulConstant = ToonProfileShadowSettings.ShadowExtinctionCoefficientInInvCentimeter;
	}
#endif

	half DynamicShadowing = 1.0f;
	FDeferredLightData LightData = GetDirectionalLightData(ScreenPosition, DynamicShadowing, DirectionalShadowSettings);

#if ENABLE_SHADOWMASKTEXTURE 
	half4 PreviewShadowMapChannelMask = half4(1.0f, 0.0f, 0.0f, 0.0f);
	DynamicShadowFactors = DecodeLightAttenuation(DynamicShadowFactors);
	#if ALLOW_STATIC_LIGHTING
		PreviewShadowMapChannelMask = UnpackShadowMapChannelMask(MobileDirectionalLight.DirectionalLightShadowMapChannelMask >> 4);
		DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);
	#else
		DynamicShadowing = DynamicShadowFactors.r;
	#endif
#endif

	half4 LightAttenuation = half4(1, 1, DynamicShadowing, DynamicShadowing);

#if TRANSLUCENCY_SH_LIGHTING
	FLightAccumulator NewLighting = GetTranslucencySHLighting(GBuffer, LightData, TranslatedWorldPosition);
#else
	FLightAccumulator NewLighting = AccumulateDynamicLighting(TranslatedWorldPosition, CameraVector, GBuffer, 1, LightData, LightAttenuation, Dither, uint2(0, 0), OutDirectionalLightShadow);
#endif
	DirectLighting = LightAccumulator_Add(DirectLighting, NewLighting);
}

/*------------------------------------------------------------------------------
	Mobile Reflection.
------------------------------------------------------------------------------*/

#if IS_MOBILE_BASE_PASS || IS_DECAL
/** Prenormalized capture of the scene that's closest to the object being rendered. */
half3 GetMobileSkyLightReflection(half3 ReflectionVector, half Roughness, half CubemapMaxMip)
{
	half AbsoluteSpecularMip = ComputeReflectionCaptureMipFromRoughness(Roughness, CubemapMaxMip);
	half4 Reflection = MobileReflectionCapture.Texture.SampleLevel(MobileReflectionCapture.TextureSampler, ReflectionVector, AbsoluteSpecularMip);
#if FORWARD_SHADING_FORCES_SKYLIGHT_CUBEMAPS_BLENDING
	float BlendFraction = MobileReflectionCapture.Params.w;
	if (BlendFraction > 0.0f)
	{
		half4 ReflectionBlend = MobileReflectionCapture.TextureBlend.SampleLevel(MobileReflectionCapture.TextureBlendSampler, ReflectionVector, AbsoluteSpecularMip);
		Reflection = lerp(Reflection, ReflectionBlend, BlendFraction.xxxx);
	}
#endif
	return Reflection.rgb * ResolvedView.SkyLightColor.rgb;
}
#endif

// Common helper function for evaluating reflection probes for mobile
half3 GetImageBasedReflectionLighting_Mobile(
	half3 ReflectionVector, 
	float3 TranslatedWorldPosition, 
	half Roughness, 
	half IndirectIrradiance, 
	half CompositeAlpha, 
	uint GridIndex
#if DIFFUSE_FROM_CAPTURE
	, half3 NormalVector,
	out half4 OutDiffuseIBL		// W value returns scale to be applied to diffuse sky lighting (diffuse from captures takes precedence over sky lighting)
#endif
	)
{
	half3 SpecularIBL = (half3)0.0f;
#if DIFFUSE_FROM_CAPTURE
	OutDiffuseIBL = half4(0.0, 0.0, 0.0, 1.0);
#endif

#if ENABLE_CLUSTERED_REFLECTION
	FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);

	SpecularIBL = CompositeReflectionCapturesAndSkylightTWS(
		CompositeAlpha,
		TranslatedWorldPosition,
		ReflectionVector,//RayDirection,
		Roughness,
		IndirectIrradiance,
		1.0f,
		0.0f,
		CulledReflectionCapturesGridHeader.NumReflectionCaptures,
		CulledReflectionCapturesGridHeader.DataStartIndex,
		0,
		true
#if DIFFUSE_FROM_CAPTURE
		, NormalVector, OutDiffuseIBL
#endif
		);

#elif MOBILE_DEFERRED_LIGHTING

	// Normalize for static skylight types which mix with lightmaps.
	bool bNormalize = ReflectionStruct.SkyLightParameters.z < 1;
	float SkyAverageBrightness = 1.0f;
	SpecularIBL = GetSkyLightReflection(ReflectionVector, Roughness, SkyAverageBrightness);
	
	if (bNormalize)
	{
		#if ALLOW_STATIC_LIGHTING
		SpecularIBL *= ComputeMixingWeight(IndirectIrradiance, SkyAverageBrightness, Roughness);
		#endif
	}

	SpecularIBL *= CompositeAlpha;
#elif IS_MOBILE_BASE_PASS || IS_DECAL

	bool UsingSkyReflection = MobileReflectionCapture.Params.y > 0.0f;
	// Normalize for static skylight types which mix with lightmaps.
	bool bNormalize = !UsingSkyReflection || MobileReflectionCapture.Params.z < 1;
	if (UsingSkyReflection)
	{
		// Apply sky colour if the reflection map is the sky.
		SpecularIBL = GetMobileSkyLightReflection(ReflectionVector, Roughness, MobileReflectionCapture.Params.y);
	}
	else
	{
		half AbsoluteSpecularMip = ComputeReflectionCaptureMipFromRoughness(Roughness, ResolvedView.ReflectionCubemapMaxMip);
		SpecularIBL = MobileReflectionCapture.Texture.SampleLevel(MobileReflectionCapture.TextureSampler, ReflectionVector, AbsoluteSpecularMip).rgb;

		half ReflectionCaptureBrightness = MobileReflectionCapture.Params.w;
		SpecularIBL = SpecularIBL * ReflectionCaptureBrightness;
	}

	if (bNormalize)
	{
		#if ALLOW_STATIC_LIGHTING
		SpecularIBL *= ComputeMixingWeight(IndirectIrradiance, MobileReflectionCapture.Params.x, Roughness);
		#endif
	}

	SpecularIBL *= CompositeAlpha;
#endif

	return SpecularIBL;
}

// Common helper function for evaluating planar reflection for mobile
half3 GetPlanarReflectionbasedReflectionLighting_Mobile(float3 TranslatedWorldPosition, half3 WorldNormal, half Roughness, half3 InSpecularIBL)
{
	half3 Out = InSpecularIBL;
#if ENABLE_PLANAR_REFLECTION || MATERIAL_PLANAR_FORWARD_REFLECTIONS
	BRANCH
	if (abs(dot(PlanarReflectionStruct.ReflectionPlane.xyz, 1)) > .0001f)
	{
		half4 PlanarReflection = GetPlanarReflection(TranslatedWorldPosition, WorldNormal, Roughness);
		// Planar reflections win over reflection environment
		Out = lerp(InSpecularIBL, PlanarReflection.rgb, PlanarReflection.a);
	}
#endif
	return Out;
}

// Version used for Substrate
float3 GetImageBasedReflectionSpecular(
	  half3 SpecularDirection, 
	  half Roughness, 
	  float3 TranslatedWorldPosition, 
	  half3 WorldNormal, 
	  half SpecularOcclusion,
	  half IndirectIrradiance, 
	  uint GridIndex)
{
	half3 Out = 0;

	// IBL reflection
	half4 DiffuseIBL = half4(0.0, 0.0, 0.0, 1.0);
	Out = GetImageBasedReflectionLighting_Mobile(
		SpecularDirection,
		TranslatedWorldPosition,
		Roughness,
		IndirectIrradiance,
		SpecularOcclusion,
		GridIndex
#if DIFFUSE_FROM_CAPTURE
		, WorldNormal
		, DiffuseIBL
#endif
	);

	// TODO: Use DiffuseIBL?

	// Planar reflection
	Out = GetPlanarReflectionbasedReflectionLighting_Mobile(TranslatedWorldPosition, WorldNormal, Roughness, Out);

	return Out;
}

// Version used for legacy shading.  Return value is scale to be applied to diffuse sky lighting.  The "DIFFUSE_FROM_CAPTURE" feature applies
// ambient diffuse light from a low mip of the reflection capture, taking precedence over sky lighting where present.
half AccumulateReflection(
	FGBufferData GBuffer, 
	half4 SvPosition,
	half3 CameraVector, 
	float3 TranslatedWorldPosition, 
	half3 ReflectionVector, 
	half IndirectIrradiance, 
	uint GridIndex, 
	inout FLightAccumulator DirectLighting)
{
	half3 SpecularIBLLighting = (half3)0.0f;

	half3 N = GBuffer.WorldNormal;
	half3 V = -CameraVector;
	half NoV = saturate(abs(dot(N, V)) + 1e-5);

	half3 TopLayerR = ReflectionVector;
	half SpecularOcclusion = GBuffer.GBufferAO;
	
	// Point lobe in off-specular peak direction
	ReflectionVector = GetOffSpecularPeakReflectionDir(N, ReflectionVector, GBuffer.Roughness);
	half RoughnessSq = GBuffer.Roughness * GBuffer.Roughness;
	SpecularOcclusion = GetSpecularOcclusion(NoV, RoughnessSq, SpecularOcclusion);

	// IBL reflection (primary)
	half4 DiffuseIBL = half4(0.0, 0.0, 0.0, 1.0);
	half3 SpecularIBL = GetImageBasedReflectionLighting_Mobile(ReflectionVector
		, TranslatedWorldPosition
		, GBuffer.Roughness
		, IndirectIrradiance
		, SpecularOcclusion
		, GridIndex
#if DIFFUSE_FROM_CAPTURE
		, GBuffer.WorldNormal
		, DiffuseIBL
#endif
	);


#if MOBILE_USE_SSR
	half4 SSR = 0;
	if (MobileSSR(GBuffer, SvPosition, TranslatedWorldPosition, ReflectionVector, SSR))
	{
		SpecularIBL.rgb = SpecularIBL.rgb * (1 - SSR.a) + SSR.rgb;
	}
#endif // MOBILE_USE_SSR

	// Planar reflection
	SpecularIBL = GetPlanarReflectionbasedReflectionLighting_Mobile(TranslatedWorldPosition, GBuffer.WorldNormal, GBuffer.Roughness, SpecularIBL);

	half3 DiffuseColor = GBuffer.DiffuseColor;
	half3 SpecularColor = GBuffer.SpecularColor;

	if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		const half ClearCoat = GBuffer.CustomData.x;
		const half ClearCoatRoughness = GBuffer.CustomData.y;

		RemapClearCoatDiffuseAndSpecularColor(GBuffer, NoV, DiffuseColor, SpecularColor);

		half F = GetEnvBRDF(0.04, ClearCoatRoughness, NoV).x;
		F *= ClearCoat;
		half LayerAttenuation = (1 - F);

		// Fc * Vis
	#if MOBILE_USE_PREINTEGRATED_GF
		half2 AB = PreIntegratedGF.SampleLevel(PreIntegratedGFSampler, float2(NoV, GBuffer.Roughness), 0).rg;
	#else
		half2 AB = EnvBRDFApproxLazarov(GBuffer.Roughness, NoV);
	#endif
		SpecularIBLLighting += SpecularIBL * LayerAttenuation * (SpecularColor * AB.x + AB.y * saturate(50 * SpecularColor.g) * (1 - ClearCoat));

		// IBL reflection (secondary)
#if DIFFUSE_FROM_CAPTURE
		half4 IgnoredSecondaryDiffuseIBL = 0;
#endif
		SpecularIBL = GetImageBasedReflectionLighting_Mobile(TopLayerR
			, TranslatedWorldPosition
			, ClearCoatRoughness
			, IndirectIrradiance
			, F * SpecularOcclusion
			, GridIndex
#if DIFFUSE_FROM_CAPTURE
			, GBuffer.WorldNormal
			, IgnoredSecondaryDiffuseIBL
#endif
		);
		SpecularIBLLighting += SpecularIBL;
	}
	else if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
	{
		//Skip IBL for Hair
	}
	else
	{
		SpecularIBLLighting += SpecularIBL * GetEnvBRDF(SpecularColor, GBuffer.Roughness, NoV);
	}

#if DIFFUSE_FROM_CAPTURE
	DiffuseIBL.xyz *= DiffuseColor * GBuffer.GBufferAO;
#endif

	LightAccumulator_AddSplit(DirectLighting, DiffuseIBL.xyz, SpecularIBLLighting, 0.0f, 1.0f, false);

	return DiffuseIBL.w;
}

/*------------------------------------------------------------------------------
	Mobile Local lights.
------------------------------------------------------------------------------*/
/**
 * Adds local lighting using the light grid, does not apply directional lights, as they are done elsewhere.
 */
#if ENABLE_CLUSTERED_LIGHTS

void AccumulateLightGridLocalLighting(const FCulledLightsGridHeader InLightGridHeader
	, FGBufferData GBuffer
	, float3 TranslatedWorldPosition
	, half3 CameraVector
	, uint EyeIndex
	, uint FirstNonSimpleLightIndex
	, half4 DynamicShadowFactors
#if !MOBILE_DEFERRED_LIGHTING
	, uint LightingChannelMask
#endif
	, inout FLightAccumulator DirectLighting)
{
	// Limit max to ForwardLightStruct.NumLocalLights.
	// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
	const uint NumLightsInGridCell = min(InLightGridHeader.NumLights, GetMaxLightsPerCell());

	LOOP
	for (uint GridLightListIndex = FirstNonSimpleLightIndex; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
	{
		const FLocalLightData LocalLight = GetLocalLightDataFromGrid(InLightGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);

#if MOBILE_DEFERRED_LIGHTING
		// The lights are sorted such that all that support clustered deferred are at the beginning, there might be others
		// (e.g., lights with dynamic shadows) so we break out when the condition fails.
		if (!UnpackIsClusteredDeferredSupported(LocalLight))
		{
			break;
		}
#endif

		// extra-early out since we know light grid is sloppy and all lights in list are radial (have a range)
		// appears useless
		if (!IsLightVisible(LocalLight, TranslatedWorldPosition))
		{
			continue;
		}

		uint LightSceneInfoExtraDataPacked = UnpackLightSceneInfoExtraDataPacked(LocalLight);

#if !MOBILE_DEFERRED_LIGHTING
		// Check lighting channels for mobile forward
		if ((UnpackLightingChannelMask(LocalLight) & LightingChannelMask) == 0)
		{
			continue;
		}
#endif

		// bits [17:16] really
		uint LightType = UnpackLightType(LightSceneInfoExtraDataPacked);

		half DynamicShadowing = 1.0f;
#if ENABLE_SHADOWMASKTEXTURE
		if (LightType == LIGHT_TYPE_SPOT || LightType == LIGHT_TYPE_POINT)
		{
			half4 PreviewShadowMapChannelMask = UnpackShadowMapChannelMask(LightSceneInfoExtraDataPacked >> 4);
			DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);
		}
#endif

		FDeferredLightData LightData = ConvertToDeferredLight(LocalLight);

		if (LightType == LIGHT_TYPE_SPOT || LightType == LIGHT_TYPE_POINT)
		{
			half Attenuation = ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);
			LightData.Color *= Attenuation;
		}

		half4 LightAttenuation = half4(1, 1, DynamicShadowing, DynamicShadowing);

		float SurfaceShadow = 0;
#if TRANSLUCENCY_SH_LIGHTING
		FLightAccumulator NewLighting = GetTranslucencySHLighting(GBuffer, LightData, TranslatedWorldPosition);
#else
		FLightAccumulator NewLighting = AccumulateDynamicLighting(TranslatedWorldPosition, CameraVector, GBuffer, 1, LightData, LightAttenuation, 0, uint2(0, 0), SurfaceShadow);
#endif
		DirectLighting = LightAccumulator_Add(DirectLighting, NewLighting);
	}
}
#endif

/*------------------------------------------------------------------------------
	Mobile Forward Decal Lighting
------------------------------------------------------------------------------*/

half3 ForwardDecalLighting(
	float3 TranslatedWorldPosition, 
	float2 ScreenPosition, 
	float SceneDepth, 
	half3 CameraVector, 
	half3 VertexNormal, 
	half3 ReflectionVector,
	half3 WorldNormal,
	half  Specular,
	half3 BaseColor,
	half  Metallic,
	half  Roughness,
	half3 EmissiveColor, 
	half  Opacity)
{
	FDirectionalShadowSettings DirectionalShadowSettings = GetDefaultDirectionalShadowSettings();
	float ShadowPositionZ = 0;
	half ShadowMap = MobileDirectionalLightCSM(ScreenPosition.xy, SceneDepth, ShadowPositionZ, DirectionalShadowSettings);
	half3 DirectionalLightDirection = MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.xyz;
	half NoL = saturate(dot(half3(WorldNormal), DirectionalLightDirection));
	
	half3 SpecularColor = ComputeF0(Specular, BaseColor, Metallic);
	half3 DiffuseColor = BaseColor - BaseColor * Metallic;

	half3 OutColor = EmissiveColor;
	OutColor += MobileDirectionalLight.DirectionalLightColor.rgb * NoL * SimpleShading(DiffuseColor, SpecularColor, max(Roughness, .04f), DirectionalLightDirection, -CameraVector, WorldNormal) * ShadowMap;

	half3 SkyDiffuse = GetSkySHDiffuseSimple(WorldNormal) * ResolvedView.SkyLightColor.rgb;
	OutColor += DiffuseColor * SkyDiffuse;

	half3 SpecularIBL = GetImageBasedReflectionSpecular(ReflectionVector, Roughness, TranslatedWorldPosition, WorldNormal, 1.0, Luminance(SkyDiffuse), 0);
	half NoV = saturate(abs(dot(WorldNormal, -CameraVector)) + 1e-5);
	OutColor += SpecularIBL * EnvBRDFApprox(SpecularColor, Roughness, NoV);

#if MATERIALBLENDING_MODULATE
	OutColor *= Opacity;
#endif

	return OutColor;
}