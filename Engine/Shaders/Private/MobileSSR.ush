// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

#include "SSRT/SSRTRayCast.ush"

#if MATERIALBLENDING_ANY_TRANSLUCENT || MOBILE_SSR_QUALITY <= 1
	#define MOBILE_SSR_VELOCITY_TEXTURE_SAMPLER
	#ifndef MOBILE_SSR_STEPS
		#define MOBILE_SSR_STEPS 8
	#endif
#else
	#define MOBILE_SSR_VELOCITY_TEXTURE_SAMPLER MobileSceneTextures.SceneVelocityTexture, MobileSceneTextures.SceneVelocityTextureSampler, 
	#ifndef MOBILE_SSR_STEPS
		#define MOBILE_SSR_STEPS 12
	#endif
#endif


#ifndef SSRData
#define SSRData MobileBasePass.SSRParams
#endif

#define SSRData_Intensity						((half)SSRData.IntensityAndExposureCorrection.x)
#define SSRData_PrevSceneColorPreExposureInv	((half)SSRData.IntensityAndExposureCorrection.y)
#define SSRData_MaxRoughness					((half)SSRData.IntensityAndExposureCorrection.z)
#define SSRData_2DivMaxRoughness				((half)SSRData.IntensityAndExposureCorrection.w)

bool MobileSSR(FGBufferData GBuffer, float4 SvPosition, float3 TranslatedWorldPosition, half3 ReflectionVector, out half4 SSR)
{
	BRANCH if(View.CameraCut == 0 && SSRData_Intensity > 0.0 && GBuffer.Roughness <= SSRData_MaxRoughness && GBuffer.ShadingModelID != 0)
	{
		float StepOffset = InterleavedGradientNoise(SvPosition.xy, SSRData.NoiseIndex);
		StepOffset -= 0.5;
		float3 HitUVz;
		float Level = 0;
		bool bHit = RayCast(SSRData.HZBTexture, SSRData.HZBSampler, TranslatedWorldPosition, ReflectionVector, GBuffer.Roughness, GBuffer.Depth,
			MOBILE_SSR_STEPS, StepOffset, SSRData.HZBUvFactorAndInvFactor, /*bDebugPrint=*/false, HitUVz, Level);

		BRANCH if(bHit)
		{
			float2 SampleUV;
			float Vignette;
			ReprojectHit(SSRData.PrevScreenPositionScaleBias, MOBILE_SSR_VELOCITY_TEXTURE_SAMPLER 
				HitUVz, SampleUV, Vignette);

			SampleUV = clamp(SampleUV, SSRData.PrevSceneColorBilinearUVMinMax.xy,  SSRData.PrevSceneColorBilinearUVMinMax.zw);
			SSR = SampleScreenColor(SSRData.SceneColor, SSRData.SceneColorSampler, SampleUV);

			half RoughnessFade = RoughnessFade = saturate(2.0 - SSRData_2DivMaxRoughness * GBuffer.Roughness);
			SSR *= Vignette * SSRData_Intensity * RoughnessFade;
			SSR.rgb *= SSRData_PrevSceneColorPreExposureInv;
			return true;
		}
	}
	SSR = 0;
	return false;
}
