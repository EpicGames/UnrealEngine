// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

/*=============================================================================
	MonteCarlo.usf: Monte Carlo integration of distributions
=============================================================================*/

// [ Duff et al. 2017, "Building an Orthonormal Basis, Revisited" ]
// Discontinuity at TangentZ.z == 0
float3x3 GetTangentBasis( float3 TangentZ )
{
	const float Sign = TangentZ.z >= 0 ? 1 : -1;
	const float a = -rcp( Sign + TangentZ.z );
	const float b = TangentZ.x * TangentZ.y * a;
	
	float3 TangentX = { 1 + Sign * a * Pow2( TangentZ.x ), Sign * b, -Sign * TangentZ.x };
	float3 TangentY = { b,  Sign + a * Pow2( TangentZ.y ), -TangentZ.y };

	return float3x3( TangentX, TangentY, TangentZ );
}

// [Frisvad 2012, "Building an Orthonormal Basis from a 3D Unit Vector Without Normalization"]
// Discontinuity at TangentZ.z < -0.9999999f
float3x3 GetTangentBasisFrisvad(float3 TangentZ)
{
	float3 TangentX;
	float3 TangentY;

	if (TangentZ.z < -0.9999999f)
	{
		TangentX = float3(0, -1, 0);
		TangentY = float3(-1, 0, 0);
	}
	else
	{
		float A = 1.0f / (1.0f + TangentZ.z);
		float B = -TangentZ.x * TangentZ.y * A;
		TangentX = float3(1.0f - TangentZ.x * TangentZ.x * A, B, -TangentZ.x);
		TangentY = float3(B, 1.0f - TangentZ.y * TangentZ.y * A, -TangentZ.y);
	}

	return float3x3( TangentX, TangentY, TangentZ );
}

float3 TangentToWorld( float3 Vec, float3 TangentZ )
{
	return mul( Vec, GetTangentBasis( TangentZ ) );
}

float3 WorldToTangent(float3 Vec, float3 TangentZ)
{
	return mul(GetTangentBasis(TangentZ), Vec);
}

float2 Hammersley( uint Index, uint NumSamples, uint2 Random )
{
	float E1 = frac( (float)Index / NumSamples + float( Random.x & 0xffff ) / (1<<16) );
	float E2 = float( reversebits(Index) ^ Random.y ) * 2.3283064365386963e-10;
	return float2( E1, E2 );
}

float2 Hammersley16( uint Index, uint NumSamples, uint2 Random )
{
	float E1 = frac( (float)Index / NumSamples + float( Random.x ) * (1.0 / 65536.0) );
	float E2 = float( ( reversebits(Index) >> 16 ) ^ Random.y ) * (1.0 / 65536.0);
	return float2( E1, E2 );
}

// http://extremelearning.com.au/a-simple-method-to-construct-isotropic-quasirandom-blue-noise-point-sequences/
float2 R2Sequence( uint Index )
{
	const float Phi = 1.324717957244746;
	const float2 a = float2( 1.0 / Phi, 1.0 / Pow2(Phi) );
	return frac( a * Index );
}

// R2 Jittered point set
// These seem to be garbage so use at your own risk. Jitter is not large enough for low sample counts. Larger jitter overlaps neighboring samples unevenly.
float2 JitteredR2( uint Index, uint NumSamples, float2 Jitter, float JitterAmount = 0.5 )
{
	const float Phi = 1.324717957244746;
	const float2 a = float2( 1.0 / Phi, 1.0 / Pow2(Phi) );
	const float d0 = 0.76;
	const float i0 = 0.7;

	return frac( a * float(Index) + ( JitterAmount * 0.5 * d0 * sqrt(PI) * rsqrt( float(NumSamples) ) ) * Jitter );
}

// R2 Jittered point sequence. Progressive
float2 JitteredR2( uint Index, float2 Jitter, float JitterAmount = 0.5 )
{
	const float Phi = 1.324717957244746;
	const float2 a = float2( 1.0 / Phi, 1.0 / Pow2(Phi) );
	const float d0 = 0.76;
	const float i0 = 0.7;

	return frac( a * Index + ( JitterAmount * 0.25 * d0 * sqrt(PI) * rsqrt( Index - i0 ) ) * Jitter );
}


///////

float2 UniformSampleDisk( float2 E )
{
	float Theta = 2 * PI * E.x;
	float Radius = sqrt( E.y );
	return Radius * float2( cos( Theta ), sin( Theta ) );
}

// Returns a point on the unit circle and a radius in z
float3 ConcentricDiskSamplingHelper(float2 E)
{
	// Rescale input from [0,1) to (-1,1). This ensures the output radius is in [0,1)
	float2 p = 2 * E - 0.99999994;
	float2 a = abs(p);
	float Lo = min(a.x, a.y);
	float Hi = max(a.x, a.y);
	float Epsilon = 5.42101086243e-20; // 2^-64 (this avoids 0/0 without changing the rest of the mapping)
	float Phi = (PI / 4) * (Lo / (Hi + Epsilon) + 2 * float(a.y >= a.x));
	float Radius = Hi;
	// copy sign bits from p
	const uint SignMask = 0x80000000;
	float2 Disk = asfloat((asuint(float2(cos(Phi), sin(Phi))) & ~SignMask) | (asuint(p) & SignMask));
	// return point on the circle as well as the radius
	return float3(Disk, Radius);
}

float2 UniformSampleDiskConcentric( float2 E )
{
	float3 Result = ConcentricDiskSamplingHelper(E);
	return Result.xy * Result.z; // uniform sampling
}

// based on the approximate equal area transform from
// http://marc-b-reynolds.github.io/math/2017/01/08/SquareDisc.html
float2 UniformSampleDiskConcentricApprox( float2 E )
{
	float2 sf = E * sqrt(2.0) - sqrt(0.5);	// map 0..1 to -sqrt(0.5)..sqrt(0.5)
	float2 sq = sf*sf;
	float root = sqrt(2.0*max(sq.x, sq.y) - min(sq.x, sq.y));
	if (sq.x > sq.y)
	{
		sf.x = sf.x > 0 ? root : -root;
	}
	else
	{
		sf.y = sf.y > 0 ? root : -root;
	}
	return sf;
}


// Based on: [Clarberg 2008, "Fast Equal-Area Mapping of the (Hemi)Sphere using SIMD"]
// Fixed sign bit for UV.y == 0 and removed branch before division by using a small epsilon
// https://fileadmin.cs.lth.se/graphics/research/papers/2008/simdmapping/clarberg_simdmapping08_preprint.pdf
float3 EquiAreaSphericalMapping(float2 UV)
{
	UV = 2 * UV - 1;
	float D = 1 - (abs(UV.x) + abs(UV.y));
	float R = 1 - abs(D);
	// Branch to avoid dividing by 0.
	// Only happens with (0.5, 0.5), usually occurs in odd number resolutions which use the very central texel
	float Phi = R == 0 ? 0 : (PI / 4) * ((abs(UV.y) - abs(UV.x)) / R + 1);
	float F = R * sqrt(2 - R * R);
	return float3(
		F * sign(UV.x) * abs(cos(Phi)),
		F * sign(UV.y) * abs(sin(Phi)),
		sign(D) * (1 - R * R)
	);
}

// Based on: [Clarberg 2008, "Fast Equal-Area Mapping of the (Hemi)Sphere using SIMD"]
// Removed branch before division by using a small epsilon
// https://fileadmin.cs.lth.se/graphics/research/papers/2008/simdmapping/clarberg_simdmapping08_preprint.pdf
float2 InverseEquiAreaSphericalMapping(float3 Direction)
{
	// Most use cases of this func generate Direction by diffing two positions and thus unnormalized
	Direction = normalize(Direction);
	
	float3 AbsDir = abs(Direction);
	float R = sqrt(1 - AbsDir.z);
	float Epsilon = 5.42101086243e-20; // 2^-64 (this avoids 0/0 without changing the rest of the mapping)
	float x = min(AbsDir.x, AbsDir.y) / (max(AbsDir.x, AbsDir.y) + Epsilon);

	// Coefficients for 6th degree minimax approximation of atan(x)*2/pi, x=[0,1].
	const float t1 = 0.406758566246788489601959989e-5f;
	const float t2 = 0.636226545274016134946890922156f;
	const float t3 = 0.61572017898280213493197203466e-2f;
	const float t4 = -0.247333733281268944196501420480f;
	const float t5 = 0.881770664775316294736387951347e-1f;
	const float t6 = 0.419038818029165735901852432784e-1f;
	const float t7 = -0.251390972343483509333252996350e-1f;

	// Polynomial approximation of atan(x)*2/pi
	float Phi = t6 + t7 * x;
	Phi = t5 + Phi * x;
	Phi = t4 + Phi * x;
	Phi = t3 + Phi * x;
	Phi = t2 + Phi * x;
	Phi = t1 + Phi * x;

	Phi = (AbsDir.x < AbsDir.y) ? 1 - Phi : Phi;
	float2 UV = float2(R - Phi * R, Phi * R);
	UV = (Direction.z < 0) ? 1 - UV.yx : UV;
	UV = asfloat(asuint(UV) ^ (asuint(Direction.xy) & 0x80000000u));
	return UV * 0.5 + 0.5;
}


// PDF = 1 / (4 * PI)
float4 UniformSampleSphere( float2 E )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = 1 - 2 * E.y;
	float SinTheta = sqrt( 1 - CosTheta * CosTheta );

	float3 H;
	H.x = SinTheta * cos( Phi );
	H.y = SinTheta * sin( Phi );
	H.z = CosTheta;

	float PDF = 1.0 / (4 * PI);

	return float4( H, PDF );
}

// PDF = 1 / (2 * PI)
float4 UniformSampleHemisphere( float2 E )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = E.y;
	float SinTheta = sqrt( 1 - CosTheta * CosTheta );

	float3 H;
	H.x = SinTheta * cos( Phi );
	H.y = SinTheta * sin( Phi );
	H.z = CosTheta;

	float PDF = 1.0 / (2 * PI);

	return float4( H, PDF );
}

// PDF = NoL / PI
float4 CosineSampleHemisphere( float2 E )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = sqrt(E.y);
	float SinTheta = sqrt(1 - CosTheta * CosTheta);

	float3 H;
	H.x = SinTheta * cos(Phi);
	H.y = SinTheta * sin(Phi);
	H.z = CosTheta;

	float PDF = CosTheta * (1.0 / PI);

	return float4(H, PDF);
}

// PDF = NoL / PI
// Simpler than cosine sampling + tangent transform if you only want a world space direction
float4 CosineSampleHemisphereWorld( float2 E, float3 N )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = 1 - 2 * E.y;
	float SinTheta = sqrt( 1 - CosTheta * CosTheta );

	float3 U;
	U.x = SinTheta * cos( Phi );
	U.y = SinTheta * sin( Phi );
	U.z = CosTheta;

	float3 V = normalize(U + N);

	float PDF = saturate(dot(V, N)) * (1.0 / PI);

	return float4( V, PDF );
}


// PDF = NoL / PI
float4 CosineSampleHemisphereConcentric(float2 E)
{
	float3 Result = ConcentricDiskSamplingHelper(E);
	float SinTheta = Result.z;
	float CosTheta = sqrt(1 - SinTheta * SinTheta);
	return float4(Result.xy * SinTheta, CosTheta, CosTheta * (1.0 / PI));
}

// PDF = NoL / PI
float4 CosineSampleHemisphere( float2 E, float3 N ) 
{
	float3 H = UniformSampleSphere( E ).xyz;
	H = normalize( N + H );

	float PDF = dot(H, N) * (1.0 /  PI);

	return float4( H, PDF );
}

float4 UniformSampleCone( float2 E, float CosThetaMax )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = lerp( CosThetaMax, 1, E.y );
	float SinTheta = sqrt( 1 - CosTheta * CosTheta );

	float3 L;
	L.x = SinTheta * cos( Phi );
	L.y = SinTheta * sin( Phi );
	L.z = CosTheta;

	float PDF = 1.0 / ( 2 * PI * (1 - CosThetaMax) );

	return float4( L, PDF );
}

// Same as the function above, but uses SinThetaMax^2 as the parameter
// so that the solid angle can be computed more accurately for very small angles
// The caller is expected to ensure that SinThetaMax2 is <= 1
float4 UniformSampleConeRobust(float2 E, float SinThetaMax2)
{
	float Phi = 2 * PI * E.x;
	// The expression 1-sqrt(1-x) is susceptible to catastrophic cancelation.
	// Instead, use a series expansion about 0 which is accurate within 10^-7
	// and much more numerically stable.
	float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);

	float CosTheta = 1 - OneMinusCosThetaMax * E.y;
	float SinTheta = sqrt(1 - CosTheta * CosTheta);

	float3 L;
	L.x = SinTheta * cos(Phi);
	L.y = SinTheta * sin(Phi);
	L.z = CosTheta;
	float PDF = 1.0 / (2 * PI * OneMinusCosThetaMax);

	return float4(L, PDF);
}

float UniformConeSolidAngle(float SinThetaMax2)
{
	float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);
	return 2 * PI * OneMinusCosThetaMax;
}

// Same as the function above, but uses a concentric mapping
float4 UniformSampleConeConcentricRobust(float2 E, float SinThetaMax2)
{
	// The expression 1-sqrt(1-x) is susceptible to catastrophic cancelation.
	// Instead, use a series expansion about 0 which is accurate within 10^-7
	// and much more numerically stable.
	float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);
	float3 Result = ConcentricDiskSamplingHelper(E);
	float SinTheta = Result.z * sqrt(SinThetaMax2);
	float CosTheta = sqrt(1 - SinTheta * SinTheta);

	float3 L = float3(Result.xy * SinTheta, CosTheta);
	float PDF = 1.0 / (2 * PI * OneMinusCosThetaMax);

	return float4(L, PDF);
}

// PDF = D * NoH / (4 * VoH)
float4 ImportanceSampleGGX( float2 E, float a2 )
{
	float Phi = 2 * PI * E.x;
	float CosTheta = sqrt( (1 - E.y) / ( 1 + (a2 - 1) * E.y ) );
	float SinTheta = sqrt( 1 - CosTheta * CosTheta );

	float3 H;
	H.x = SinTheta * cos( Phi );
	H.y = SinTheta * sin( Phi );
	H.z = CosTheta;
	
	float d = ( CosTheta * a2 - CosTheta ) * CosTheta + 1;
	float D = a2 / ( PI*d*d );
	float PDF = D * CosTheta;

	return float4( H, PDF );
}

#ifndef GGX_BOUNDED_VNDF_SAMPLING
// "Bounded VNDF Sampling for Smith–GGX Reflections"
// Kenta Eto & Yusuke Tokuyoshi - Siggraph Asia 2023
// This paper further improves the method of Dupuy and Benyoub in the case where the microfacet normal is used for reflection only.
// It allows many fewer rays to be rejected at high roughness which improves quality. This is particularly noticeable
// in multi-bounce reflections of rough metals because longer paths can be followed without early termination.
#define GGX_BOUNDED_VNDF_SAMPLING 1
#endif

float VisibleGGXPDF(float3 V, float3 H, float a2, bool bLimitVDNFToReflection = true)
{
	float NoV = V.z;
	float NoH = H.z;
	float VoH = dot(V, H);

	float d = (NoH * a2 - NoH) * NoH + 1;
	float D = a2 / (PI*d*d);
	float k = 1.0;
#if GGX_BOUNDED_VNDF_SAMPLING
	if (bLimitVDNFToReflection)
	{
		float s = 1.0f + length(V.xy);
		float s2 = s * s;
		k = (s2 - a2 * s2) / (s2 + a2 * V.z * V.z); // Eq. 5
	}
#endif
	float PDF = 2 * VoH * D / (k * NoV + sqrt(NoV * (NoV - NoV * a2) + a2));
	return PDF;
}

float VisibleGGXPDF_aniso(float3 V, float3 H, float2 Alpha, bool bLimitVDNFToReflection = true)
{
	float NoV = V.z;
	float NoH = H.z;
	float VoH = dot(V, H);
	float a2 = Alpha.x * Alpha.y;
	float3 Hs = float3(Alpha.y * H.x, Alpha.x * H.y, a2 * NoH);
	float S = dot(Hs, Hs);
	float D = (1.0f / PI) * a2 * Square(a2 / S);
	float LenV = length(float3(V.x * Alpha.x, V.y * Alpha.y, NoV));
	float k = 1.0;
#if GGX_BOUNDED_VNDF_SAMPLING
	if (bLimitVDNFToReflection)
	{
		float a = saturate(min(Alpha.x, Alpha.y));
		float s = 1.0f + length(V.xy);
		float ka2 = a * a, s2 = s * s;
		k = (s2 - ka2 * s2) / (s2 + ka2 * V.z * V.z); // Eq. 5
	}
#endif
	float Pdf = (2 * D * VoH) / (k * NoV + LenV);
	return Pdf;
}

// PDF = G_SmithV * VoH * D / NoV / (4 * VoH)
// PDF = G_SmithV * D / (4 * NoV)
float4 ImportanceSampleVisibleGGX(float2 E, float2 Alpha, float3 V, bool bLimitVDNFToReflection = true)
{
	// stretch
	float3 Vh = normalize(float3(Alpha * V.xy, V.z));

	// "Sampling Visible GGX Normals with Spherical Caps"
	// Jonathan Dupuy & Anis Benyoub - High Performance Graphics 2023
	float Phi = (2 * PI) * E.x;
	float k = 1.0;
#if GGX_BOUNDED_VNDF_SAMPLING
	if (bLimitVDNFToReflection)
	{
		// If we know we will be reflecting the view vector around the sampled micronormal, we can
		// tweak the range a bit more to eliminate some of the vectors that will point below the horizon
		float a = saturate(min(Alpha.x, Alpha.y));
		float s = 1.0 + length(V.xy);
		float a2 = a * a, s2 = s * s;
		k = (s2 - a2 * s2) / (s2 + a2 * V.z * V.z);
	}
#endif
	float Z = lerp(1.0, -k * Vh.z, E.y);
	float SinTheta = sqrt(saturate(1 - Z * Z));
	float X = SinTheta * cos(Phi);
	float Y = SinTheta * sin(Phi);
	float3 H = float3(X, Y, Z) + Vh;

	// unstretch
	H = normalize(float3(Alpha * H.xy, max(0.0, H.z)));

	return float4(H, VisibleGGXPDF_aniso(V, H, Alpha, bLimitVDNFToReflection));
}

// Multiple importance sampling balance heuristic
// [Veach 1997, "Robust Monte Carlo Methods for Light Transport Simulation"]
float MISWeightBalanced(float Pdf, float OtherPdf)
{
	// The straightforward implementation is prone to numerical overflow, divisions by 0
	// and does not work well with +inf inputs.
	// return Pdf / (Pdf + OtherPdf);

	// We want this function to have the following properties:
	//  0 <= w(a,b) <= 1 for all possible positive floats a and b (including 0 and +inf)
	//  w(a, b) + w(b, a) == 1.0

	// The formulation below is much more stable across the range of all possible inputs
	// and guarantees the sum always adds up to 1.0.

	// Evaluate the expression using the ratio of the smaller value to the bigger one for greater
	// numerical stability. The math would also work using the ratio of bigger to smaller value,
	// which would underflow less but would make the weights asymmetric. Underflow to 0 is not a
	// bad property to have in rendering application as it ensures more weights are exactly 0
	// which allows some evaluations to be skipped.
	float X = min(Pdf, OtherPdf) / max(Pdf, OtherPdf); // This ratio is guaranteed to be in [0,1]
	float Y = Pdf == OtherPdf ? 1.0 : X; // Guard against NaNs from 0/0 and Inf/Inf
	float M = rcp(1.0 + Y);
	return Pdf > OtherPdf ? M : 1.0 - M; // This ensures exchanging arguments will produce values that add back up to 1.0 exactly
}

// Multiple importance sampling power heuristic of two functions with a power of two. 
// [Veach 1997, "Robust Monte Carlo Methods for Light Transport Simulation"]
float MISWeightPower(float Pdf, float OtherPdf)
{
	// Naive code (which can overflow, divide by 0, etc ..)
	// return Pdf * Pdf / (Pdf * Pdf + OtherPdf * OtherPdf);

	// See function above for the explanation of how this works
	float X = min(Pdf, OtherPdf) / max(Pdf, OtherPdf); // This ratio is guaranteed to be in [0,1]
	float Y = Pdf == OtherPdf ? 1.0 : X; // Guard against NaNs from 0/0 and Inf/Inf
	float M = rcp(1.0 + Y * Y);
	return Pdf > OtherPdf ? M : 1.0 - M; // This ensures exchanging arguments will produce values that add back up to 1.0 exactly
}

// Takes as input the sample weight and pdf for a certain lobe of a mixed model, together with the probability of picking that lobe
// This function then updates a running total Weight and Pdf value that represents the overall contribution of the BxDF
// This function should be called when a BxDF is made up of multiple lobes combined with a sum to correctly account for the probability
// of sampling directions via all lobes.
// NOTE: this function also contains special logic to handle cases with infinite pdfs cleanly
void AddLobeWithMIS(inout float3 Weight, inout float Pdf, float3 LobeWeight, float LobePdf, float LobeProb)
{
	const float MinLobeProb = 1.1754943508e-38; // smallest normal float
	if (LobeProb > MinLobeProb)
	{
		LobePdf *= LobeProb;
		LobeWeight *= rcp(LobeProb);
		Weight = lerp(Weight, LobeWeight, MISWeightBalanced(LobePdf, Pdf));
		Pdf += LobePdf;
	}
}

// When sampling from discrete CDFs, it can be convenient to re-use the random number by rescaling it
// This function assumes that RandVal is in the interval: [LowerBound, UpperBound) and returns a value in [0,1)
float RescaleRandomNumber(float RandVal, float LowerBound, float UpperBound)
{
	const float OneMinusEpsilon = 0.99999994; // 32-bit float just before 1.0
	return min((RandVal - LowerBound) / (UpperBound - LowerBound), OneMinusEpsilon);
}


// Compute PDF of reflection (2 * dot( V, H ) * H - V).
// [Heitz 2014, "Importance Sampling Microfacet-Based BSDFs using the Distribution of Visible Normals"]
float RayPDFToReflectionRayPDF(float VoH, float RayPDF)
{
	float ReflectPDF = RayPDF / (4.0 * saturate(VoH));

	return ReflectPDF;
}
