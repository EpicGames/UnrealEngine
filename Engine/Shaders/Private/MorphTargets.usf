// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MorphTargets.usf: Compute shader for calculating Morph Targets.
=============================================================================*/


#include "Common.ush"
#include "BitPacking.ush"
#include "ComputeShaderUtils.ush"

////////////////////////////////////////////////////////////////

#define GMorphTargetDispatchBatchSize	128
#define BATCH_SIZE						64
#define NUM_BATCH_HEADER_DWORDS			10
#define INDEX_MAX_BITS					31
#define POSITION_MAX_BITS				28
#define TANGENTZ_MAX_BITS				16

RWBuffer<uint> MorphVertexBuffer;
ByteAddressBuffer MorphDataBuffer;

//TODO: use a buffer instead of packing into constant buffer?
uint4 MorphTargetBatchOffsets[GMorphTargetDispatchBatchSize / 4];
uint4 MorphTargetGroupOffsets[GMorphTargetDispatchBatchSize / 4];
float4 MorphTargetWeights[GMorphTargetDispatchBatchSize / 4];

float4 PositionScale;
float2 Precision;
uint NumGroups;

groupshared uint LocalGroupOffsets[GMorphTargetDispatchBatchSize];

void LoadLocalData(uint GroupIndex)
{
	if (GroupIndex < GMorphTargetDispatchBatchSize / 4)
	{
		uint GroupSharedIndex = GroupIndex * 4;
		uint4 Offset = MorphTargetGroupOffsets[GroupIndex];

		for (int i = 0; i < 4; ++i)
		{
			LocalGroupOffsets[GroupSharedIndex + i] = Offset[i];
		}
	}

	GroupMemoryBarrierWithGroupSync();
}

uint GroupIndexToMorphTargetLocalIndex(uint GroupThreadIndex)
{
	uint Idx = 0;
	uint Width = GMorphTargetDispatchBatchSize >> 1;

	UNROLL
	while (Width > 0)
	{
		Idx += (GroupThreadIndex >= LocalGroupOffsets[Idx + Width]) ? Width : 0;
		Width = Width >> 1;
	}

	return Idx;
}

struct FBatchHeader
{
	uint	DataOffset;
	uint	NumElements;
	bool	bTangents;

	uint	IndexBits;
	uint3	PositionBits;
	uint3	TangentZBits;

	uint	IndexMin;
	int3	PositionMin;
	int3	TangentZMin;
};

FBatchHeader GetBatchHeader(uint BatchIndex)
{
	const uint BaseAddress = BatchIndex * (NUM_BATCH_HEADER_DWORDS * 4);
	uint4 Data0 = MorphDataBuffer.Load4(BaseAddress);
	uint2 Data1 = MorphDataBuffer.Load2(BaseAddress + 16);

	FBatchHeader Header		= (FBatchHeader)0;
	Header.DataOffset		= Data0.x;
	Header.IndexBits		= BitFieldExtractU32(Data0.y, 5, 0);
	Header.PositionBits.x	= BitFieldExtractU32(Data0.y, 5, 5);
	Header.PositionBits.y	= BitFieldExtractU32(Data0.y, 5, 10);
	Header.PositionBits.z	= BitFieldExtractU32(Data0.y, 5, 15);
	Header.bTangents		= BitFieldExtractU32(Data0.y, 1, 20);
	Header.NumElements		= Data0.y >> 21;
	
	Header.IndexMin			= Data0.z;
	Header.PositionMin		= int3(Data0.w, Data1);

	if (Header.bTangents)
	{
		// TODO: Don't even store this. Need to handle two different batch header sizes.
		uint4 Data2				= MorphDataBuffer.Load4(BaseAddress + 24);
		Header.TangentZBits.x	= BitFieldExtractU32(Data2.x, 5, 0);
		Header.TangentZBits.y	= BitFieldExtractU32(Data2.x, 5, 5);
		Header.TangentZBits.z	= BitFieldExtractU32(Data2.x, 5, 10);
		Header.TangentZMin		= int3(Data2.yzw);
	}
	
	return Header;
}

[numthreads(BATCH_SIZE, 1, 1)]
void GPUMorphUpdateCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	LoadLocalData(GroupThreadIndex);

	const uint GroupIndex = GetUnWrappedDispatchGroupId(GroupId);
	if (GroupIndex >= NumGroups)
	{
		return;
	}

	const uint MorphTargetLocalIndex = GroupIndexToMorphTargetLocalIndex(GroupIndex);

	const uint BatchStartIndex = MorphTargetBatchOffsets[MorphTargetLocalIndex >> 2][MorphTargetLocalIndex & 3u];			//TODO: use a buffer instead of packing into constant buffer?
	const uint GroupOffset = GroupIndex - LocalGroupOffsets[MorphTargetLocalIndex];
	const FBatchHeader BatchHeader = GetBatchHeader(BatchStartIndex + GroupIndex - LocalGroupOffsets[MorphTargetLocalIndex]);

	if (GroupThreadIndex < BatchHeader.NumElements)
	{
		const float Weight			= MorphTargetWeights[MorphTargetLocalIndex >> 2][MorphTargetLocalIndex & 3u];			//TODO: use a buffer instead of packing into constant buffer?
		const uint NumPositionBits	= BatchHeader.PositionBits.x + BatchHeader.PositionBits.y + BatchHeader.PositionBits.z;
		const uint NumTangentBits	= BatchHeader.TangentZBits.x + BatchHeader.TangentZBits.y + BatchHeader.TangentZBits.z;
		const uint StrideInBits		= BatchHeader.IndexBits + NumPositionBits + (BatchHeader.bTangents ? NumTangentBits : 0);

		const uint CompileTimeMaxBits = INDEX_MAX_BITS + 3 * POSITION_MAX_BITS + 3 * TANGENTZ_MAX_BITS;
		FBitStreamReaderState InputStream = BitStreamReader_Create_Aligned(BatchHeader.DataOffset, GroupThreadIndex * StrideInBits, CompileTimeMaxBits);
		const uint DestVertexIndex = BitStreamReader_Read_RO(MorphDataBuffer, InputStream, BatchHeader.IndexBits, INDEX_MAX_BITS) + BatchHeader.IndexMin + GroupThreadIndex;
		
		const int3 LocalPosition = BitStreamReader_Read3_RO(MorphDataBuffer, InputStream, BatchHeader.PositionBits, POSITION_MAX_BITS);
		const float3 PositionDelta = float3(LocalPosition + BatchHeader.PositionMin) * Precision.x;
		const int3 QuantizedPositionOffset = (int3)round(PositionDelta * Weight * PositionScale.xyz);

		const uint DestVertexAddress = DestVertexIndex * 6;
		InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 0], asuint(QuantizedPositionOffset.x));
		InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 1], asuint(QuantizedPositionOffset.y));
		InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 2], asuint(QuantizedPositionOffset.z));

		if (BatchHeader.bTangents)
		{
			const int3 LocalTangent = BitStreamReader_Read3_RO(MorphDataBuffer, InputStream, BatchHeader.TangentZBits, TANGENTZ_MAX_BITS);
			const float3 TangentZDelta = float3(LocalTangent + BatchHeader.TangentZMin) * Precision.y;
			const int3 QuantizedTangentZOffset = (int3)round(TangentZDelta * Weight * PositionScale.w);
			InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 3], asuint(QuantizedTangentZOffset.x));
			InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 4], asuint(QuantizedTangentZOffset.y));
			InterlockedAdd(MorphVertexBuffer[DestVertexAddress + 5], asuint(QuantizedTangentZOffset.z));
		}
	}
}

uint NumVertices;

[numthreads(64, 1, 1)]
void GPUMorphNormalizeCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint DestVertexIndex = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, 64);
	if (DestVertexIndex >= NumVertices)
	{
		return;
	}

	const uint DestVertexAddress = DestVertexIndex * 6;
	const float3 Position = float3(int3(MorphVertexBuffer[DestVertexAddress + 0], MorphVertexBuffer[DestVertexAddress + 1], MorphVertexBuffer[DestVertexAddress + 2])) * PositionScale.xyz;

	MorphVertexBuffer[DestVertexAddress + 0] = asuint(Position.x);
	MorphVertexBuffer[DestVertexAddress + 1] = asuint(Position.y);
	MorphVertexBuffer[DestVertexAddress + 2] = asuint(Position.z);
	
	float3 TangentZ = float3(int3(MorphVertexBuffer[DestVertexAddress + 3], MorphVertexBuffer[DestVertexAddress + 4], MorphVertexBuffer[DestVertexAddress + 5])) * PositionScale.w;
	MorphVertexBuffer[DestVertexAddress + 3] = asuint(TangentZ.x);
	MorphVertexBuffer[DestVertexAddress + 4] = asuint(TangentZ.y);
	MorphVertexBuffer[DestVertexAddress + 5] = asuint(TangentZ.z);
}