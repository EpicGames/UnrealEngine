// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MortonCode.ush: Morton code lookup functions
=============================================================================*/

#pragma once

uint MortonCode2(uint x)
{
	x &= 0x0000ffff;
	x = (x ^ (x << 8)) & 0x00ff00ff;
	x = (x ^ (x << 4)) & 0x0f0f0f0f;
	x = (x ^ (x << 2)) & 0x33333333;
	x = (x ^ (x << 1)) & 0x55555555;
	return x;
}

// Encodes two 16-bit integers into one 32-bit morton code
uint MortonEncode(uint2 Pixel)
{
	uint Morton = MortonCode2(Pixel.x) | (MortonCode2(Pixel.y) << 1);
	return Morton;
}

uint ReverseMortonCode2(uint x)
{
	x &= 0x55555555;
	x = (x ^ (x >> 1)) & 0x33333333;
	x = (x ^ (x >> 2)) & 0x0f0f0f0f;
	x = (x ^ (x >> 4)) & 0x00ff00ff;
	x = (x ^ (x >> 8)) & 0x0000ffff;
	return x;
}

// Decodes one 32-bit morton code into two 16-bit integers
uint2 MortonDecode(uint Morton)
{
	uint2 Pixel = uint2(ReverseMortonCode2(Morton), ReverseMortonCode2(Morton >> 1));
	return Pixel;
}

uint MortonCode3(uint x)
{
	x &= 0x000003ff;
	x = (x ^ (x << 16)) & 0xff0000ff;
	x = (x ^ (x << 8)) & 0x0300f00f;
	x = (x ^ (x << 4)) & 0x030c30c3;
	x = (x ^ (x << 2)) & 0x09249249;
	return x;
}

uint ReverseMortonCode3(uint x)
{
	x &= 0x09249249;
	x = (x ^ (x >> 2)) & 0x030c30c3;
	x = (x ^ (x >> 4)) & 0x0300f00f;
	x = (x ^ (x >> 8)) & 0xff0000ff;
	x = (x ^ (x >> 16)) & 0x000003ff;
	return x;
}
