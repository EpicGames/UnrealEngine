// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE DIM_BLUR_DIRECTIONS


#include "MotionBlurCommon.ush"
#include "../LensDistortion.ush"
#include "../Random.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"


//------------------------------------------------------- DEBUG

// Debug the input scene color resolution.
#define DEBUG_OVERLAY_INPUT_RES 0

// Debug the gathered resolution.
#define DEBUG_OVERLAY_GATHER_RES 0

// Debug the tile classification
#define DEBUG_OVERLAY_TILE_CLASSIFICATION 0

// Debug whether the full res center is fecthed or not.
#define DEBUG_OVERLAY_SKIP_CENTER 0

// Debug the number of samples
//	1: show number of samples
//  2: show the density of samples (lower at half res)
#define DEBUG_OVERLAY_SAMPLES 0


//------------------------------------------------------- CONFIG

#if DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_GATHER_HALF_RES
	#define CONFIG_IS_HALF_RES 1

#elif DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_GATHER_FULL_RES
	#define CONFIG_IS_HALF_RES 0

#elif DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_HALF_RES
	#define CONFIG_IS_HALF_RES 1

#elif DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES
	#define CONFIG_IS_HALF_RES 0

#elif DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_2_VELOCITY_FULL_RES
	#define CONFIG_IS_HALF_RES 0

#else
	#error unknown tile classification
#endif

// from the paper: We use SOFT Z EXTENT = 1mm to 10cm for our results
#define SOFT_Z_EXTENT 1

// Whether should interleave the half res motion blur with the full res to reduce the size of the grain on screen.
#define CONFIG_SHUFFLE_HALF_RES 0

// Whether should do post motion blur translucency.
#define CONFIG_POST_MOTIONBLUR_TRANSLUCENCY 1

// Save memory bandwidth by not fetching full res center if output is fully gathered.
#if COMPILER_SUPPORTS_WAVE_MINMAX
	#define CONFIG_SKIP_CENTER 1
#else
	#define CONFIG_SKIP_CENTER 0
#endif

#define CONFIG_SCENE_COLOR_ALPHA (DIM_ALPHA_CHANNEL)


//------------------------------------------------------- CONSTANTS

#if CONFIG_IS_HALF_RES
	#define TILE_SIZE (VELOCITY_FILTER_TILE_SIZE / 2)
#else
	#define TILE_SIZE (VELOCITY_FILTER_TILE_SIZE)
#endif


//------------------------------------------------------- PARAMETERS

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Velocity)
SCREEN_PASS_TEXTURE_VIEWPORT(VelocityTile)
FScreenTransform ColorToVelocity;
FScreenTransform SeparateTranslucencyUVToViewportUV;
FScreenTransform ViewportUVToSeparateTranslucencyUV;
uint MaxSampleCount;
uint OutputMip1;
uint OutputMip2;
uint bLensDistortion;

uint TileListOffset;
StructuredBuffer<uint> TileListsBuffer;
StructuredBuffer<uint> TileListsSizeBuffer;

Texture2D<mb_half4> ColorTexture;
Texture2D<mb_half3> VelocityFlatTexture;
Texture2D VelocityTileTextures_Textures_0;
Texture2D VelocityTileTextures_Textures_1;
Texture2D<mb_half4> HalfResMotionBlurTexture;

SamplerState ColorSampler;
SamplerState VelocitySampler;
SamplerState VelocityTileSampler;
SamplerState VelocityFlatSampler;
SamplerState DepthSampler;

Texture2D<float2> UndistortingDisplacementTexture;
SamplerState      UndistortingDisplacementSampler;

Texture2D TranslucencyTexture;
SamplerState TranslucencySampler;
FScreenTransform ColorToTranslucency;
float2 TranslucencyUVMin;
float2 TranslucencyUVMax;
float2 TranslucencyExtentInverse;

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedVelocitySampler VelocitySampler
	#define SharedVelocityTileSampler VelocitySampler
	#define SharedVelocityFlatSampler VelocitySampler
	#define SharedDepthSampler VelocitySampler
#else
	#define SharedVelocitySampler VelocitySampler
	#define SharedVelocityTileSampler VelocityTileSampler
	#define SharedVelocityFlatSampler VelocityFlatSampler
	#define SharedDepthSampler DepthSampler
#endif

RWTexture2D<mb_half4> SceneColorOutputMip0;
RWTexture2D<mb_half4> SceneColorOutputMip1;
RWTexture2D<mb_half4> SceneColorOutputMip2;
RWTexture2D<float4> DebugOutput;


//------------------------------------------------------- LDS

#if !PLATFORM_SUPPORTS_WAVE_BROADCAST

groupshared mb_half4 SharedArray0[TILE_SIZE * TILE_SIZE];

#if CONFIG_SCENE_COLOR_ALPHA
groupshared mb_half  SharedArray1[TILE_SIZE * TILE_SIZE];
#endif

#endif // !PLATFORM_SUPPORTS_WAVE_BROADCAST

#if !COMPILER_SUPPORTS_WAVE_VOTE

groupshared uint SharedSampleCount;
groupshared uint SharedFastPath;

#endif


//------------------------------------------------------- FUNCTIONS

float NormalizeAccumulator(float Accumulator, mb_half AccumulatorWeight)
{
	return -min(-Accumulator * rcp(AccumulatorWeight), mb_half(0.0));
}

float4 NormalizeAccumulator(float4 Accumulator, float AccumulatorWeight)
{
	return -min(-Accumulator * rcp(AccumulatorWeight), float(0.0));
}

#if CONFIG_MOTION_BLUR_COMPILE_FP16

mb_half NormalizeAccumulator(mb_half Accumulator, mb_half AccumulatorWeight)
{
	return -min(-Accumulator * rcp(AccumulatorWeight), mb_half(0.0));
}

mb_half4 NormalizeAccumulator(mb_half4 Accumulator, mb_half AccumulatorWeight)
{
	return -min(-Accumulator * rcp(AccumulatorWeight), mb_half(0.0));
}

#endif // CONFIG_MOTION_BLUR_COMPILE_FP16

void NormalizeAccumulatorWithHoleFill(
	mb_half4 Color,
	mb_half ColorWeight,
	mb_half HoleFillWeight,
	mb_half DepthAware,
	mb_half4 HoleFillingColor,
	mb_half InvSampleCount,
	out mb_half4 ColorOutput,
	out mb_half OpacityOutput,
	out bool bValidColorOutput)
{
	OpacityOutput = min(saturate(ColorWeight * InvSampleCount * mb_half(2)), (DepthAware * InvSampleCount * mb_half(2)));
	
	mb_half4 FinalAccumulator = Color + HoleFillWeight * HoleFillingColor;
	
	ColorOutput = NormalizeAccumulator(FinalAccumulator, ColorWeight + HoleFillWeight);
	bValidColorOutput = (ColorWeight + HoleFillWeight) > mb_half(0.0);
}


// Compute the weight of the sample for hole filling.
mb_half ComputeSampleHoleFillWeight(mb_half CenterDepth, mb_half SampleDepth, mb_half DepthScale)
{
	return saturate(DepthScale * (SampleDepth - CenterDepth));
}

// Computes the contribution weight of one sample.
mb_half ComputeSampleConvolutionWeight(
	mb_half SampleDepth, mb_half SampleSpreadLength, mb_half SampleVelocityAngle,
	mb_half OffsetLength, mb_half BlurAngle, mb_half PixelToSampleScale)
{
	// Compare the length 
	mb_half SpreadWeights = saturate(PixelToSampleScale * SampleSpreadLength - max(OffsetLength - mb_half(1), mb_half(0)));
	
	// Compare the directions
	#if CONFIG_MAX_RANGE_SIZE > 1
		mb_half DirectionWeights = saturate(mb_half(1.0) - max(GetPolarRelativeAngle(SampleVelocityAngle, BlurAngle) - mb_half(0.1), mb_half(0.0)) * mb_half(4.0));
	#else
		mb_half DirectionWeights = mb_half(1.0);
	#endif

	return SpreadWeights * DirectionWeights;
}

// Selectively computes the contribution weight of the center or the sample based on whether sample is respectively behind center or not
mb_half ComputeCenterOrSampleWeight(
	mb_half CenterDepth, mb_half CenterSpreadLength, mb_half CenterVelocityAngle,
	mb_half SampleDepth, mb_half SampleSpreadLength, mb_half SampleVelocityAngle,
	mb_half OffsetLength, mb_half BlurAngle, mb_half PixelToSampleScale, mb_half DepthScale)
{
	// Compute weight to use the center data if center is closer than the sample.
	mb_half CenterWeight = saturate(0.5 + DepthScale * (SampleDepth - CenterDepth));
	
	// Compute weight to use the sample data if sample is closer than the center.
	mb_half SampleWeight = saturate(0.5 - DepthScale * (SampleDepth - CenterDepth));

	mb_half CenterConvolutionWeight = ComputeSampleConvolutionWeight(
		CenterDepth, CenterSpreadLength, CenterVelocityAngle,
		OffsetLength, BlurAngle, PixelToSampleScale);
	
	mb_half SampleConvolutionWeight = ComputeSampleConvolutionWeight(
		SampleDepth, SampleSpreadLength, SampleVelocityAngle,
		OffsetLength, BlurAngle, PixelToSampleScale);

	return CenterWeight * CenterConvolutionWeight + SampleWeight * SampleConvolutionWeight;
}

// TODO: move that to velocity flatten.
mb_half GetVelocityLengthPixels(mb_half2 EncodedVelocity)
{
	// 11:11:10  (VelocityLength, VelocityAngle, Depth)
	return EncodedVelocity.x;
}

float2 ApplyLensDistortionOnTranslucencyUV(float2 SeparateTranslucencyUV)
{
	float2 DistortedViewportUV = ApplyScreenTransform(SeparateTranslucencyUV, SeparateTranslucencyUVToViewportUV);
	float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);
	
	return ApplyScreenTransform(UndistortedViewportUV, ViewportUVToSeparateTranslucencyUV);
}


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Debug = 0;
	mb_half4 DebugOverlay = mb_half(1.0).xxxx;
	
	uint PackedGroupOffset = TileListsBuffer[GroupId + TileListOffset];
	uint2 GroupOffset = uint2(PackedGroupOffset, PackedGroupOffset >> 16) & 0xFFFF;
	
	const bool bIsHalfRes = CONFIG_IS_HALF_RES ? true : false;

	uint2 DispatchThreadId = (
		ZOrder2D(GroupThreadIndex, log2(TILE_SIZE)) +
		GroupOffset * uint2(TILE_SIZE, TILE_SIZE));

	uint2 iColorPixelPos = DispatchThreadId + Color_ViewportMin;

	float2 ColorUV;
	BRANCH
	if (bIsHalfRes)
	{
		ColorUV = (float2(DispatchThreadId * 2 + Color_ViewportMin) + 1.0) * Color_ExtentInverse;
	}
	else
	{
		ColorUV = (float2(iColorPixelPos) + 0.5) * Color_ExtentInverse;
	}

	const float PixelToTileScale = rcp(float(VELOCITY_FLATTEN_TILE_SIZE));

	float Random  = InterleavedGradientNoise(iColorPixelPos, 0);
	float Random2 = InterleavedGradientNoise(iColorPixelPos, 1);

	// [-0.25, 0.25]
	float2 TileJitter = (float2(Random, Random2) - 0.5) * 0.5;

	// Map color UV to velocity UV space.
	float2 VelocityUV = ApplyScreenTransform(ColorUV, ColorToVelocity);
	VelocityUV = clamp(VelocityUV, Velocity_UVViewportBilinearMin, Velocity_UVViewportBilinearMax);

	// Map velocity UV to velocity tile UV space with jitter.
	float2 NearestVelocitySvPosition = floor(VelocityUV * Velocity_Extent) + 0.5;
	float2 VelocityTileUV = ((NearestVelocitySvPosition - Velocity_ViewportMin) * PixelToTileScale + TileJitter) * VelocityTile_ExtentInverse;

	// Velocity tile UV originates at [0,0]; only need to clamp max.
	VelocityTileUV = min(VelocityTileUV, VelocityTile_UVViewportBilinearMax);

	FVelocityRange VelocityRange = DecodeVelocityRange(
		VelocityTileTextures_Textures_0.SampleLevel(SharedVelocityTileSampler, VelocityTileUV, 0),
		VelocityTileTextures_Textures_1.SampleLevel(SharedVelocityTileSampler, VelocityTileUV, 0));

	float2 MinVelocityPixels = VelocityRange.Min;
	float2 Max0VelocityPixels = VelocityRange.Max[0];
	#if CONFIG_MAX_RANGE_SIZE > 1
		float2 Max1VelocityPixels = VelocityRange.Max[1];
	#endif

	// Compute how many sample should be applied.
	float MipLevel0;
	float MipLevel1;
	uint SampleCount;
	float InvSampleCount;
	bool bSkip;
	bool bFastPath;
	#if CONFIG_MAX_RANGE_SIZE > 1
		bool bDoOneDirectionOnly;
	#else
		const bool bDoOneDirectionOnly = true;
	#endif
	{
		const uint SampleCountFactor = 4;

		float MaxPixelLength0 = length(Max0VelocityPixels);

		uint RecommendedSampleCount = clamp(SampleCountFactor * ceil(MaxPixelLength0 * rcp(float(SampleCountFactor))), SampleCountFactor, MaxSampleCount);
		
		bFastPath = length2(MinVelocityPixels) > 0.4 * (MaxPixelLength0 * MaxPixelLength0);

		#if COMPILER_SUPPORTS_WAVE_MINMAX && COMPILER_SUPPORTS_WAVE_VOTE
		{
			SampleCount = ToScalarMemory(WaveActiveMax(RecommendedSampleCount));
			bFastPath = WaveActiveAllTrue(bFastPath);
		}
		#else
		{
			if (GroupThreadIndex == 0)
			{
				SharedSampleCount = 0;
				SharedFastPath = 0;
			}
			GroupMemoryBarrierWithGroupSync();

			InterlockedMax(SharedSampleCount, RecommendedSampleCount);
			InterlockedAdd(SharedFastPath, bFastPath ? 1 : 0);

			GroupMemoryBarrierWithGroupSync();
			SampleCount = ToScalarMemory(SharedSampleCount);
			bFastPath = (SharedFastPath == (TILE_SIZE * TILE_SIZE));
		}
		#endif
	
		InvSampleCount = ToScalarMemory(rcp(float(SampleCount)));
		bSkip = MaxPixelLength0 < CONFIG_MINIMAL_PIXEL_VELOCITY;

		#if CONFIG_MAX_RANGE_SIZE > 1
		{
			float MaxPixelLength1 = length(Max1VelocityPixels);
			bDoOneDirectionOnly = MaxPixelLength1 < CONFIG_MINIMAL_PIXEL_VELOCITY;
			if (bDoOneDirectionOnly)
			{
				MipLevel0 = saturate(MaxPixelLength0 * InvSampleCount - 1.0);
				MipLevel1 = MipLevel0;
				Max1VelocityPixels = Max0VelocityPixels;
			}
			else
			{
				MipLevel0 = saturate(MaxPixelLength0 * (InvSampleCount * 2.0) - 1.0);
				MipLevel1 = saturate(MaxPixelLength1 * (InvSampleCount * 2.0) - 1.0);
			}
		}
		#else
		{
			MipLevel0 = saturate(MaxPixelLength0 * InvSampleCount - 1.0);
			MipLevel1 = MipLevel0;
		}
		#endif
		
		#if DEBUG_OVERLAY_SAMPLES == 1
			DebugOverlay = lerp(mb_half4(0.5, 1.0, 0.5, 1.0), mb_half4(1.0, 0.5, 0.5, 1.0), float(SampleCount) / float(MaxSampleCount));
		#elif DEBUG_OVERLAY_SAMPLES == 2
			DebugOverlay = lerp(mb_half4(0.5, 1.0, 0.5, 1.0), mb_half4(1.0, 0.5, 0.5, 1.0), float(SampleCount) / float(MaxSampleCount * (bIsHalfRes ? 4 : 1)));
		#endif
	}

	mb_half2 SearchVector0 = mb_half2(Max0VelocityPixels * Color_ExtentInverse.xy);

	mb_half4 MotionBlurColor;
	mb_half FullResBlend;
	
	BRANCH
	if (bSkip)
	{
		MotionBlurColor = mb_half(0.0).xxxx;
		FullResBlend = 1.0;
		
		#if DEBUG_OVERLAY_TILE_CLASSIFICATION
			DebugOverlay = mb_half4(0.5, 0.5, 1.0, 1.0);
		#endif

		#if DEBUG_OVERLAY_SAMPLES
			DebugOverlay = mb_half4(0.5, 1.0, 0.5, 1.0);
		#endif
	}
	#if DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_GATHER_HALF_RES || DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_GATHER_FULL_RES
	else
	#else
	else if (bFastPath)
	#endif
	{
		mb_half4 ColorAccum = 0;
		
		LOOP
		for (uint i = 0; i < SampleCount; i += 4)
		{
			UNROLL_N(2)
			for (uint j = 0; j < 2; j ++)
			{
				float2 OffsetLength = float(i / 2 + j) + (0.5 + float2(Random - 0.5, 0.5 - Random));
				float2 OffsetFraction = OffsetLength * (2.0 * InvSampleCount);

				float2 SampleUV[2];
				SampleUV[0] = ColorUV + OffsetFraction.x * SearchVector0;
				SampleUV[1] = ColorUV - OffsetFraction.y * SearchVector0;

				SampleUV[0] = clamp(SampleUV[0], Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);
				SampleUV[1] = clamp(SampleUV[1], Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);

				ColorAccum += ColorTexture.SampleLevel(ColorSampler, SampleUV[0], MipLevel0);
				ColorAccum += ColorTexture.SampleLevel(ColorSampler, SampleUV[1], MipLevel0);
			}
		}

		MotionBlurColor = ColorAccum * InvSampleCount;
		FullResBlend = 0.0;
		
		#if DEBUG_OVERLAY_TILE_CLASSIFICATION
			DebugOverlay = mb_half4(0.5, 1.0, 0.5, 1.0);
		#endif
			
		#if DEBUG_OVERLAY_INPUT_RES
		{
			mb_half4 DebugMipLevel = lerp(mb_half4(1.0, 0.5, 0.5, 1.0), mb_half4(0.5, 1.0, 0.5, 1.0), mb_half(MipLevel0));
			MotionBlurColor *= DebugMipLevel;
		}
		#endif
	}
	#if DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_HALF_RES || DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES || DIM_TILE_CLASSIFICATION == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_2_VELOCITY_FULL_RES
	else
	{
		const uint TotalSteps = SampleCount / 2;
		const uint DirectionCount = CONFIG_MAX_RANGE_SIZE;
		const uint StepPerDirectionCount = TotalSteps / DirectionCount;
		const uint SamplePerDirection = SampleCount / DirectionCount;

		mb_half3 CenterVelocityDepth = VelocityFlatTexture.SampleLevel(SharedVelocityFlatSampler, VelocityUV, 0).xyz;
		mb_half  CenterDepth = CenterVelocityDepth.z;
		mb_half  CenterVelocityLength = GetVelocityLengthPixels(CenterVelocityDepth.xy);
		mb_half  CenterVelocityAngle = CenterVelocityDepth.y * (2.0 * PI) - PI;
		
		#if CONFIG_MAX_RANGE_SIZE > 1
			mb_half4 HoleFillColor = 0;
			mb_half  HoleFillColorWeight = 0;
		#endif

		mb_half4 DirectionalColor[CONFIG_MAX_RANGE_SIZE];
		mb_half  DirectionalColorWeight[CONFIG_MAX_RANGE_SIZE];
		
		#if CONFIG_MAX_RANGE_SIZE > 1
			mb_half HoleFillWeightAccum[CONFIG_MAX_RANGE_SIZE];

			mb_half DepthAccum[CONFIG_MAX_RANGE_SIZE];
			mb_half DepthSquareAccum[CONFIG_MAX_RANGE_SIZE];
			mb_half DepthAwareWeight[CONFIG_MAX_RANGE_SIZE];
		#endif

		// Iterate over the different directions.
		UNROLL_N(CONFIG_MAX_RANGE_SIZE)
		for (uint DirectionId = 0; DirectionId < CONFIG_MAX_RANGE_SIZE; DirectionId++)
		{
			float PixelToSampleScale = TotalSteps * rsqrt(dot(Max0VelocityPixels, Max0VelocityPixels));	
			float2 SearchVector = SearchVector0;
			float BlurAngle = CartesianToPolar(Max0VelocityPixels).y;
			float MipLevel = MipLevel0;

			bool bAccumulateHoleFillColor = true;

			#if CONFIG_MAX_RANGE_SIZE > 1
			if (DirectionId == 1)
			{
				PixelToSampleScale = TotalSteps * rsqrt(dot(Max1VelocityPixels, Max1VelocityPixels));	
				SearchVector = Max1VelocityPixels * Color_ExtentInverse.xy;
				BlurAngle = CartesianToPolar(Max1VelocityPixels).y;
				bAccumulateHoleFillColor = HoleFillColorWeight == 0.0;
				MipLevel = MipLevel1;
			}
			#endif
		
			DirectionalColor[DirectionId] = 0;
			DirectionalColorWeight[DirectionId] = 0;
			
			#if CONFIG_MAX_RANGE_SIZE > 1
			{
				HoleFillWeightAccum[DirectionId] = 0;

				DepthAccum[DirectionId] = 0;
				DepthSquareAccum[DirectionId] = 0;
				DepthAwareWeight[DirectionId] = 0;
			}
			#endif
			
			// Iterate over steps of 2 samples in each directions.
			LOOP
			for (uint StepId = DirectionId; StepId < TotalSteps; StepId += CONFIG_MAX_RANGE_SIZE)
			{
				float2 SampleUV[2];
				mb_half4 SampleColor[2];
				mb_half SampleDepth[2];
				mb_half SampleVelocityLength[2];
				mb_half ConvolutionWeight[2];
				mb_half HoleFillingWeight[2];

				mb_half2 OffsetLength = mb_half(StepId + 0.5) + mb_half2(Random - 0.5, 0.5 - Random) * (bDoOneDirectionOnly ? mb_half(1.0) : mb_half(2.0));
				mb_half2 OffsetFraction = OffsetLength * rcp(mb_half(TotalSteps));

				mb_half WeightOffsetLength = mb_half(DirectionId + StepId) + 0.5;

				SampleUV[0] = ColorUV + OffsetFraction.x * SearchVector;
				SampleUV[1] = ColorUV - OffsetFraction.y * SearchVector;

				SampleUV[0] = clamp(SampleUV[0], Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);
				SampleUV[1] = clamp(SampleUV[1], Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);

				UNROLL
				for (uint j = 0; j < 2; j++)
				{
					mb_half3 SampleVelocityDepth = VelocityFlatTexture.SampleLevel(
						SharedVelocityFlatSampler, ApplyScreenTransform(SampleUV[j], ColorToVelocity), 0).xyz;

					SampleColor[j] = ColorTexture.SampleLevel(ColorSampler, SampleUV[j], MipLevel);
					SampleDepth[j] = mb_half(SampleVelocityDepth.z);

					// Decode
					SampleVelocityDepth.x = GetVelocityLengthPixels(SampleVelocityDepth.x); // TODO: move in velocity flatten
					SampleVelocityDepth.y = SampleVelocityDepth.y * mb_half(2.0 * PI) - mb_half(PI);

					// in pixels
					SampleVelocityLength[j] = SampleVelocityDepth.x;
					
					ConvolutionWeight[j] = ComputeSampleConvolutionWeight(
						SampleVelocityDepth.z, SampleVelocityDepth.x, SampleVelocityDepth.y,
						OffsetLength.x, BlurAngle, PixelToSampleScale);

					HoleFillingWeight[j] = ComputeCenterOrSampleWeight(
						CenterDepth, CenterVelocityLength, CenterVelocityAngle,
						SampleVelocityDepth.z, SampleVelocityDepth.x, SampleVelocityDepth.y,
						WeightOffsetLength, BlurAngle, PixelToSampleScale, SOFT_Z_EXTENT);
				}
		
				{
					bool2 Mirror = bool2(SampleDepth[0] > SampleDepth[1], SampleVelocityLength[0] < SampleVelocityLength[1]);
					HoleFillingWeight[0] = all(Mirror) ? HoleFillingWeight[1] : HoleFillingWeight[0];
					HoleFillingWeight[1] = any(Mirror) ? HoleFillingWeight[1] : HoleFillingWeight[0];
				}
				
				#if CONFIG_MAX_RANGE_SIZE > 1
				{
					DirectionalColor[DirectionId] += ConvolutionWeight[0] * SampleColor[0] + ConvolutionWeight[1] * SampleColor[1];
					DirectionalColorWeight[DirectionId] += ConvolutionWeight[0] + ConvolutionWeight[1];
				
					DepthAccum[DirectionId] += HoleFillingWeight[0] * SampleDepth[0] + HoleFillingWeight[1] * SampleDepth[1];
					DepthSquareAccum[DirectionId] += HoleFillingWeight[0] * Square(SampleDepth[0]) + HoleFillingWeight[1] * Square(SampleDepth[1]);
					DepthAwareWeight[DirectionId] += HoleFillingWeight[0] + HoleFillingWeight[1];

				
					// Measure how much should hole fill DirectionalColor with.
					{
						mb_half HoleFillingWeight0 = saturate(HoleFillingWeight[0] - ConvolutionWeight[0]);
						mb_half HoleFillingWeight1 = saturate(HoleFillingWeight[1] - ConvolutionWeight[1]);

						//mb_half HoleFillingWeight = ComputeSampleHoleFillWeight(CenterDepth, SampleDepth[j], SOFT_Z_EXTENT);
						
						HoleFillWeightAccum[DirectionId] += HoleFillingWeight0 + HoleFillingWeight1;
					
						// Build a hole filling along the major directional blur.
						if (bAccumulateHoleFillColor)
						{
							HoleFillColor += HoleFillingWeight0 * SampleColor[0] + HoleFillingWeight1 * SampleColor[1];
							HoleFillColorWeight += HoleFillingWeight0 + HoleFillingWeight1;
						}
					}

				}
				#else
				{
					DirectionalColor[DirectionId] += HoleFillingWeight[0] * SampleColor[0] + HoleFillingWeight[1] * SampleColor[1];
					DirectionalColorWeight[DirectionId] += HoleFillingWeight[0] + HoleFillingWeight[1];
				}
				#endif
			} // for (uint StepId = 0; StepId < StepCount; StepId += CONFIG_MAX_RANGE_SIZE)

			#if DEBUG_OVERLAY_INPUT_RES
			{
				mb_half4 DebugMipLevel = lerp(mb_half4(1.0, 0.5, 0.5, 1.0), mb_half4(0.5, 1.0, 0.5, 1.0), mb_half(MipLevel0));
				DirectionalColor[DirectionId] *= DebugMipLevel;
			}
			#endif
		} // for (uint DirectionId = 0; DirectionId < CONFIG_MAX_RANGE_SIZE; DirectionId++)
		
		#if CONFIG_MAX_RANGE_SIZE > 1
		{
			mb_half InvDirectionSampleCount = InvSampleCount * 2.0;
			if (bDoOneDirectionOnly)
			{
				DirectionalColor[0] += DirectionalColor[1];
				DirectionalColorWeight[0] += DirectionalColorWeight[1];
				HoleFillWeightAccum[0] += HoleFillWeightAccum[1];
		
				DirectionalColor[1] = mb_half(0.0);
				DirectionalColorWeight[1] = mb_half(0.0);
				HoleFillWeightAccum[1] = mb_half(0.0);
		
				InvDirectionSampleCount = InvSampleCount;
			}
		
			mb_half Velocity1Translucency;
			{
				mb_half AvgDepthSquare0 = NormalizeAccumulator(DepthSquareAccum[0], mb_half(DepthAwareWeight[0]));
				mb_half AvgDepth0 = NormalizeAccumulator(DepthAccum[0], mb_half(DepthAwareWeight[0]));
				mb_half AvgDepth1 = NormalizeAccumulator(DepthAccum[1], mb_half(DepthAwareWeight[1]));

				mb_half Variance0 = AvgDepthSquare0 - Square(AvgDepth0);

				Velocity1Translucency = mb_half(saturate(2.0 * Variance0 / (Variance0 + Square(AvgDepth1 - AvgDepth0))));
			}

			mb_half4 NormalizedHoleFillColor0 = NormalizeAccumulator(HoleFillColor, HoleFillColorWeight);
			mb_half4 NormalizedColor0;
			mb_half Opacity0;
			bool bValidColorOutput0;
			NormalizeAccumulatorWithHoleFill(
				DirectionalColor[0],
				DirectionalColorWeight[0],
				HoleFillWeightAccum[0],
				DepthAwareWeight[0],
				NormalizedHoleFillColor0,
				InvDirectionSampleCount,
				/* out */ NormalizedColor0,
				/* out */ Opacity0,
				/* out */ bValidColorOutput0);

			mb_half4 NormalizedHoleFillColor1 = lerp(NormalizedColor0, NormalizedHoleFillColor0, bValidColorOutput0 ? Velocity1Translucency : 1.0);
			mb_half4 NormalizedColor1;
			mb_half Opacity1;
			bool bValidColorOutput1;
			NormalizeAccumulatorWithHoleFill(
				DirectionalColor[1],
				DirectionalColorWeight[1],
				HoleFillWeightAccum[1],
				DepthAwareWeight[1],
				NormalizedHoleFillColor1,
				InvDirectionSampleCount,
				/* out */ NormalizedColor1,
				/* out */ Opacity1,
				/* out */ bValidColorOutput1);
		
			Opacity1 *= saturate(1.0 - Velocity1Translucency);

			mb_half CenterColorContribution = saturate(1.0 - Opacity0) * saturate(1.0 - Opacity1);
			mb_half InvTotalWeight = rcp(Opacity0 + Opacity1);

			MotionBlurColor = (NormalizedColor0 * Opacity0 + NormalizedColor1 * Opacity1) * -min(-InvTotalWeight * (1.0 - CenterColorContribution), 0.0);
			FullResBlend = CenterColorContribution;
			
			#if DEBUG_OVERLAY_TILE_CLASSIFICATION
				DebugOverlay = mb_half4(1.0, 0.5, 0.5, 1.0);
			#endif
		}
		#else // CONFIG_MAX_RANGE_SIZE == 1
		{
			const uint DirectionId = 0;

			DirectionalColor[DirectionId] *= mb_half(0.5) / mb_half(StepPerDirectionCount);
			DirectionalColorWeight[DirectionId] *= mb_half(0.5) / mb_half(StepPerDirectionCount);

			MotionBlurColor = DirectionalColor[DirectionId];
			FullResBlend = (mb_half(1.0) - DirectionalColorWeight[DirectionId]);
			
			#if DEBUG_OVERLAY_TILE_CLASSIFICATION
				DebugOverlay = mb_half4(1.0, 1.0, 0.5, 1.0);
			#endif
		}
		#endif
	}
	#endif

	#if CONFIG_SKIP_CENTER
		bool bFetchCenter = WaveActiveAnyTrue(FullResBlend > 0.0);
	#else
		const bool bFetchCenter = true;
	#endif

	#if DEBUG_OVERLAY_GATHER_RES
	{
		mb_half4 DebugHalfResGatherLevel = lerp(mb_half4(1.0, 0.5, 0.5, 1.0), mb_half4(0.5, 1.0, 0.5, 1.0), bIsHalfRes ? mb_half(1.0) : mb_half(0.0));
		MotionBlurColor *= DebugHalfResGatherLevel;
	}
	#endif

	#if DEBUG_OVERLAY_SKIP_CENTER
		DebugOverlay = lerp(mb_half4(1.0, 0.5, 0.5, 1.0), mb_half4(0.5, 1.0, 0.5, 1.0), bFetchCenter ? mb_half(0.0) : mb_half(1.0));
	#endif

	BRANCH
	if (bIsHalfRes)
	{
		mb_short2 OutputPixelCoord = mb_short2(uint2(ColorUV * Color_Extent)) & mb_short(~0x1);
		mb_short2 OutputPixelCoord0 = min(OutputPixelCoord + mb_short2(0, 0), mb_short2(Color_ViewportMax - 1));
		mb_short2 OutputPixelCoord1 = min(OutputPixelCoord + mb_short2(1, 0), mb_short2(Color_ViewportMax - 1));
		mb_short2 OutputPixelCoord2 = min(OutputPixelCoord + mb_short2(0, 1), mb_short2(Color_ViewportMax - 1));
		mb_short2 OutputPixelCoord3 = min(OutputPixelCoord + mb_short2(1, 1), mb_short2(Color_ViewportMax - 1));

		#if 1
			float2 PostMotionBlurTranslucencyUV0 = ApplyScreenTransform((float2(uint2(OutputPixelCoord0)) + 0.5) * Color_ExtentInverse, ColorToTranslucency);
			float2 PostMotionBlurTranslucencyUV1 = ApplyScreenTransform((float2(uint2(OutputPixelCoord1)) + 0.5) * Color_ExtentInverse, ColorToTranslucency);
			float2 PostMotionBlurTranslucencyUV2 = ApplyScreenTransform((float2(uint2(OutputPixelCoord2)) + 0.5) * Color_ExtentInverse, ColorToTranslucency);
			float2 PostMotionBlurTranslucencyUV3 = ApplyScreenTransform((float2(uint2(OutputPixelCoord3)) + 0.5) * Color_ExtentInverse, ColorToTranslucency);
		#else
			float2 PostMotionBlurTranslucencyUV = ApplyScreenTransform(ColorUV, ColorToTranslucency);
			float2 PostMotionBlurTranslucencyUV0 = PostMotionBlurTranslucencyUV + float2(-0.5, -0.5) * TranslucencyExtentInverse;
			float2 PostMotionBlurTranslucencyUV1 = PostMotionBlurTranslucencyUV + float2(+0.5, -0.5) * TranslucencyExtentInverse;
			float2 PostMotionBlurTranslucencyUV2 = PostMotionBlurTranslucencyUV + float2(-0.5, +0.5) * TranslucencyExtentInverse;
			float2 PostMotionBlurTranslucencyUV3 = PostMotionBlurTranslucencyUV + float2(+0.5, +0.5) * TranslucencyExtentInverse;
		#endif

		BRANCH
		if (bLensDistortion)
		{
			PostMotionBlurTranslucencyUV0 = ApplyLensDistortionOnTranslucencyUV(PostMotionBlurTranslucencyUV0);
			PostMotionBlurTranslucencyUV1 = ApplyLensDistortionOnTranslucencyUV(PostMotionBlurTranslucencyUV1);
			PostMotionBlurTranslucencyUV2 = ApplyLensDistortionOnTranslucencyUV(PostMotionBlurTranslucencyUV2);
			PostMotionBlurTranslucencyUV3 = ApplyLensDistortionOnTranslucencyUV(PostMotionBlurTranslucencyUV3);
		}
			
		PostMotionBlurTranslucencyUV0 = clamp(PostMotionBlurTranslucencyUV0, TranslucencyUVMin, TranslucencyUVMax);
		PostMotionBlurTranslucencyUV1 = clamp(PostMotionBlurTranslucencyUV1, TranslucencyUVMin, TranslucencyUVMax);
		PostMotionBlurTranslucencyUV2 = clamp(PostMotionBlurTranslucencyUV2, TranslucencyUVMin, TranslucencyUVMax);
		PostMotionBlurTranslucencyUV3 = clamp(PostMotionBlurTranslucencyUV3, TranslucencyUVMin, TranslucencyUVMax);
		
		mb_half4 CenterColor0;
		mb_half4 CenterColor1;
		mb_half4 CenterColor2;
		mb_half4 CenterColor3;
		
		mb_half4 PostMotionBlurTranslucency0;
		mb_half4 PostMotionBlurTranslucency1;
		mb_half4 PostMotionBlurTranslucency2;
		mb_half4 PostMotionBlurTranslucency3;
		
		// Fetch center and motion blur translucency with overlapped texture fetches
		BRANCH
		if (bFetchCenter)
		{
			CenterColor0 = ColorTexture[OutputPixelCoord0];
			CenterColor1 = ColorTexture[OutputPixelCoord1];
			CenterColor2 = ColorTexture[OutputPixelCoord2];
			CenterColor3 = ColorTexture[OutputPixelCoord3];
		
			PostMotionBlurTranslucency0 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV0, 0);
			PostMotionBlurTranslucency1 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV1, 0);
			PostMotionBlurTranslucency2 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV2, 0);
			PostMotionBlurTranslucency3 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV3, 0);
		}
		else
		{
			CenterColor0 = mb_half(0.0).xxxx;
			CenterColor1 = mb_half(0.0).xxxx;
			CenterColor2 = mb_half(0.0).xxxx;
			CenterColor3 = mb_half(0.0).xxxx;
		
			PostMotionBlurTranslucency0 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV0, 0);
			PostMotionBlurTranslucency1 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV1, 0);
			PostMotionBlurTranslucency2 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV2, 0);
			PostMotionBlurTranslucency3 = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV3, 0);
		}
		
		#if DEBUG_OVERLAY_INPUT_RES
		{
			CenterColor0 *= mb_half4(1.0, 0.5, 0.5, 1.0);
			CenterColor1 *= mb_half4(1.0, 0.5, 0.5, 1.0);
			CenterColor2 *= mb_half4(1.0, 0.5, 0.5, 1.0);
			CenterColor3 *= mb_half4(1.0, 0.5, 0.5, 1.0);
		}
		#endif

		// Swizzle the half res 2x2 motion blur color across the full res 4x4 quad
		//  0  0  1  1      0  1  0  1
		//  0  0  1  1  ->  2  3  2  3
		//  2  2  3  3      0  1  0  1
		//  2  2  3  3      2  3  2  3
		mb_half4 MotionBlurColor0;
		mb_half4 MotionBlurColor1;
		mb_half4 MotionBlurColor2;
		mb_half4 MotionBlurColor3;
		mb_half FullResBlend0;
		mb_half FullResBlend1;
		mb_half FullResBlend2;
		mb_half FullResBlend3;
		#if CONFIG_SHUFFLE_HALF_RES && PLATFORM_SUPPORTS_WAVE_BROADCAST
		{
			const uint LaneGroupSize = 4;
			const uint InnerLaneGroupSize = 1;

			const FWaveBroadcastSettings Broadcast0 = InitWaveBroadcastLaneGroup(LaneGroupSize, InnerLaneGroupSize, /* InnerLaneGroupId = */ 0);
			const FWaveBroadcastSettings Broadcast1 = InitWaveBroadcastLaneGroup(LaneGroupSize, InnerLaneGroupSize, /* InnerLaneGroupId = */ 1);
			const FWaveBroadcastSettings Broadcast2 = InitWaveBroadcastLaneGroup(LaneGroupSize, InnerLaneGroupSize, /* InnerLaneGroupId = */ 2);
			const FWaveBroadcastSettings Broadcast3 = InitWaveBroadcastLaneGroup(LaneGroupSize, InnerLaneGroupSize, /* InnerLaneGroupId = */ 3);
			
			MotionBlurColor0 = WaveBroadcast(Broadcast0, MotionBlurColor);
			MotionBlurColor1 = WaveBroadcast(Broadcast1, MotionBlurColor);
			MotionBlurColor2 = WaveBroadcast(Broadcast2, MotionBlurColor);
			MotionBlurColor3 = WaveBroadcast(Broadcast3, MotionBlurColor);
		
			FullResBlend0 = WaveBroadcast(Broadcast0, FullResBlend);
			FullResBlend1 = WaveBroadcast(Broadcast1, FullResBlend);
			FullResBlend2 = WaveBroadcast(Broadcast2, FullResBlend);
			FullResBlend3 = WaveBroadcast(Broadcast3, FullResBlend);
		}
		#elif CONFIG_SHUFFLE_HALF_RES && !PLATFORM_SUPPORTS_WAVE_BROADCAST
		{
			#if CONFIG_SCENE_COLOR_ALPHA
				SharedArray0[GroupThreadIndex] = MotionBlurColor;
				SharedArray1[GroupThreadIndex] = FullResBlend;
			#else
				SharedArray0[GroupThreadIndex] = float4(MotionBlurColor.rgb, FullResBlend);
			#endif

			MotionBlurColor0 = SharedArray0[(GroupThreadIndex & (~0x3)) | 0x0];
			MotionBlurColor1 = SharedArray0[(GroupThreadIndex & (~0x3)) | 0x1];
			MotionBlurColor2 = SharedArray0[(GroupThreadIndex & (~0x3)) | 0x2];
			MotionBlurColor3 = SharedArray0[(GroupThreadIndex & (~0x3)) | 0x3];
		
			#if CONFIG_SCENE_COLOR_ALPHA
				FullResBlend0 = SharedArray1[(GroupThreadIndex & (~0x3)) | 0x0];
				FullResBlend1 = SharedArray1[(GroupThreadIndex & (~0x3)) | 0x1];
				FullResBlend2 = SharedArray1[(GroupThreadIndex & (~0x3)) | 0x2];
				FullResBlend3 = SharedArray1[(GroupThreadIndex & (~0x3)) | 0x3];
			#else
				FullResBlend0 = MotionBlurColor0.a;
				FullResBlend1 = MotionBlurColor1.a;
				FullResBlend2 = MotionBlurColor2.a;
				FullResBlend3 = MotionBlurColor3.a;
			#endif
		}
		#else // !CONFIG_SHUFFLE_HALF_RES
		{
			MotionBlurColor0 = MotionBlurColor;
			MotionBlurColor1 = MotionBlurColor;
			MotionBlurColor2 = MotionBlurColor;
			MotionBlurColor3 = MotionBlurColor;
		
			FullResBlend0 = FullResBlend;
			FullResBlend1 = FullResBlend;
			FullResBlend2 = FullResBlend;
			FullResBlend3 = FullResBlend;
		}
		#endif

		// Blend full res and motion blur
		mb_half4 OutputColor0 = CenterColor0 * FullResBlend0 + MotionBlurColor0;
		mb_half4 OutputColor1 = CenterColor1 * FullResBlend1 + MotionBlurColor1;
		mb_half4 OutputColor2 = CenterColor2 * FullResBlend2 + MotionBlurColor2;
		mb_half4 OutputColor3 = CenterColor3 * FullResBlend3 + MotionBlurColor3;
		
		// Adds debug overlay
		#if DEBUG_OVERLAY_TILE_CLASSIFICATION || DEBUG_OVERLAY_SKIP_CENTER || DEBUG_OVERLAY_SAMPLES
		{
			OutputColor0 *= DebugOverlay;
			OutputColor1 *= DebugOverlay;
			OutputColor2 *= DebugOverlay;
			OutputColor3 *= DebugOverlay;
		}
		#endif

		// Blend post motion blur translucency
		#if CONFIG_POST_MOTIONBLUR_TRANSLUCENCY
		{
			OutputColor0.rgb = OutputColor0.rgb * PostMotionBlurTranslucency0.a + PostMotionBlurTranslucency0.rgb;
			OutputColor1.rgb = OutputColor1.rgb * PostMotionBlurTranslucency1.a + PostMotionBlurTranslucency1.rgb;
			OutputColor2.rgb = OutputColor2.rgb * PostMotionBlurTranslucency2.a + PostMotionBlurTranslucency2.rgb;
			OutputColor3.rgb = OutputColor3.rgb * PostMotionBlurTranslucency3.a + PostMotionBlurTranslucency3.rgb;

			#if CONFIG_SCENE_COLOR_ALPHA
				OutputColor0.a = OutputColor0.a * PostMotionBlurTranslucency0.a;
				OutputColor1.a = OutputColor1.a * PostMotionBlurTranslucency1.a;
				OutputColor2.a = OutputColor2.a * PostMotionBlurTranslucency2.a;
				OutputColor3.a = OutputColor3.a * PostMotionBlurTranslucency3.a;
			#endif
		}
		#endif
		
		// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
		// (0.995 chosen to accommodate handling of 254/255)
		#if CONFIG_SCENE_COLOR_ALPHA
		{
			OutputColor0[3] = select(OutputColor0[3] > mb_half(0.995), mb_half(1.0), OutputColor0[3]);
			OutputColor1[3] = select(OutputColor1[3] > mb_half(0.995), mb_half(1.0), OutputColor1[3]);
			OutputColor2[3] = select(OutputColor2[3] > mb_half(0.995), mb_half(1.0), OutputColor2[3]);
			OutputColor3[3] = select(OutputColor3[3] > mb_half(0.995), mb_half(1.0), OutputColor3[3]);

			OutputColor0[3] = select(OutputColor0[3] < mb_half(0.005), mb_half(0.0), OutputColor0[3]);
			OutputColor1[3] = select(OutputColor1[3] < mb_half(0.005), mb_half(0.0), OutputColor1[3]);
			OutputColor2[3] = select(OutputColor2[3] < mb_half(0.005), mb_half(0.0), OutputColor2[3]);
			OutputColor3[3] = select(OutputColor3[3] < mb_half(0.005), mb_half(0.0), OutputColor3[3]);
		}
		#else
		{
			OutputColor0.a = 0;
			OutputColor1.a = 0;
			OutputColor2.a = 0;
			OutputColor3.a = 0;
		}
		#endif
	
		// Compute the half res.
		mb_half4 HalfResOutput = mb_half(0.25) * (OutputColor0 + OutputColor1 + OutputColor2 + OutputColor3);
		
		// Compute the quarter res
		mb_half4 QuarterResOutput;
		BRANCH
		if (OutputMip2)
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST
		{
			FWaveBroadcastSettings Horizontal = InitWaveXorButterfly(/* XorButterFly = */ 0x1);
			FWaveBroadcastSettings Vertical = InitWaveXorButterfly(/* XorButterFly = */ 0x2);
			
			QuarterResOutput = HalfResOutput * mb_half(0.25);
			QuarterResOutput += WaveBroadcast(Horizontal, QuarterResOutput);
			QuarterResOutput += WaveBroadcast(Vertical, QuarterResOutput);
		}
		#else
		{
			QuarterResOutput = HalfResOutput * mb_half(0.25);

			SharedArray0[GroupThreadIndex] = QuarterResOutput;
			QuarterResOutput += SharedArray0[GroupThreadIndex ^ 0x1];
			
			SharedArray0[GroupThreadIndex] = QuarterResOutput;
			QuarterResOutput += SharedArray0[GroupThreadIndex ^ 0x2];
		}
		#endif
		else
		{
			QuarterResOutput = 0.0;
		}
		
		// Needed to avoid crash in shader compiler of Xclipse driver
		#if !CONFIG_SCENE_COLOR_ALPHA
			QuarterResOutput.a = 0;
		#endif
		bool bIsValid = all(OutputPixelCoord < mb_short2(Color_ViewportMax));
		mb_short2 OutputPixelCoord0Mip0 = (bIsValid) ? OutputPixelCoord : mb_short(~0).xx;
		mb_short2 OutputPixelCoord1Mip0 = (bIsValid) ? OutputPixelCoord1 : mb_short(~0).xx;
		mb_short2 OutputPixelCoord2Mip0 = (bIsValid) ? OutputPixelCoord2 : mb_short(~0).xx;
		mb_short2 OutputPixelCoord3Mip0 = (bIsValid) ? OutputPixelCoord3 : mb_short(~0).xx;
		mb_short2 OutputPixelCoordMip1 = select(and(bIsValid, OutputMip1 != 0), (OutputPixelCoord >> mb_short(1)), mb_short(~0).xx);
		mb_short2 OutputPixelCoordMip2 = select(and(bIsValid, and(OutputMip2 != 0, (OutputPixelCoord & 0x3) == 0)), (OutputPixelCoord >> mb_short(2)), mb_short(~0).xx);

		SceneColorOutputMip0[OutputPixelCoord0Mip0] = OutputColor0;
		SceneColorOutputMip0[OutputPixelCoord1Mip0] = OutputColor1;
		SceneColorOutputMip0[OutputPixelCoord2Mip0] = OutputColor2;
		SceneColorOutputMip0[OutputPixelCoord3Mip0] = OutputColor3;
			
		SceneColorOutputMip1[OutputPixelCoordMip1] = HalfResOutput;
		SceneColorOutputMip2[OutputPixelCoordMip2] = QuarterResOutput;
		
		#if DEBUG_MOTION_BLUR_OUTPUT
			DebugOutput[OutputPixelCoord0Mip0] = Debug;
			DebugOutput[OutputPixelCoord1Mip0] = Debug;
			DebugOutput[OutputPixelCoord2Mip0] = Debug;
			DebugOutput[OutputPixelCoord3Mip0] = Debug;
		#endif
	}
	else
	{
		mb_short2 OutputPixelCoord = mb_short2(ColorUV * Color_Extent);

		float2 PostMotionBlurTranslucencyUV = ApplyScreenTransform(ColorUV, ColorToTranslucency);
		BRANCH
		if (bLensDistortion)
		{
			PostMotionBlurTranslucencyUV = ApplyLensDistortionOnTranslucencyUV(PostMotionBlurTranslucencyUV);
		}
		PostMotionBlurTranslucencyUV = clamp(PostMotionBlurTranslucencyUV, TranslucencyUVMin, TranslucencyUVMax);
		
		mb_half4 CenterColor;
		mb_half4 PostMotionBlurTranslucency;
		
		// Fetch center and motion blur translucency with overlapped texture fetches
		BRANCH
		if (bFetchCenter)
		{
			CenterColor = ColorTexture[min(OutputPixelCoord, mb_short2(Color_ViewportMax - 1))];
			PostMotionBlurTranslucency = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV, 0);
		}
		else
		{
			CenterColor = mb_half(0.0).xxxx;
			PostMotionBlurTranslucency = TranslucencyTexture.SampleLevel(TranslucencySampler, PostMotionBlurTranslucencyUV, 0);
		}
		
		#if DEBUG_OVERLAY_INPUT_RES
		{
			CenterColor *= mb_half4(1.0, 0.5, 0.5, 1.0);
		}
		#endif

		// Blend full res and motion blur
		mb_half4 OutputColor = CenterColor * FullResBlend + MotionBlurColor;
		
		// Adds debug overlay
		#if DEBUG_OVERLAY_TILE_CLASSIFICATION || DEBUG_OVERLAY_SKIP_CENTER || DEBUG_OVERLAY_SAMPLES
		{
			OutputColor *= DebugOverlay;
		}
		#endif

		// Blend post motion blur translucency
		#if CONFIG_POST_MOTIONBLUR_TRANSLUCENCY
		{
			OutputColor.rgb = OutputColor.rgb * PostMotionBlurTranslucency.a + PostMotionBlurTranslucency.rgb;
			
			#if CONFIG_SCENE_COLOR_ALPHA
				OutputColor.a = OutputColor.a * PostMotionBlurTranslucency.a;
			#endif
		}
		#endif

		// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
		// (0.995 chosen to accommodate handling of 254/255)
		#if CONFIG_SCENE_COLOR_ALPHA
		{
			OutputColor[3] = select(OutputColor[3] > mb_half(0.995), mb_half(1.0), OutputColor[3]);
			OutputColor[3] = select(OutputColor[3] < mb_half(0.005), mb_half(0.0), OutputColor[3]);
		}
		#else
		{
			OutputColor.a = 0;
		}
		#endif
	
		mb_half4 HalfResOutput;
		BRANCH
		if (OutputMip1 || OutputMip2)
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST
		{
			FWaveBroadcastSettings Horizontal = InitWaveXorButterfly(/* XorButterFly = */ 0x1);
			FWaveBroadcastSettings Vertical = InitWaveXorButterfly(/* XorButterFly = */ 0x2);
			
			HalfResOutput = OutputColor * mb_half(0.25);
			HalfResOutput += WaveBroadcast(Horizontal, HalfResOutput);
			HalfResOutput += WaveBroadcast(Vertical, HalfResOutput);
		}
		#else
		{
			HalfResOutput = OutputColor * mb_half(0.25);

			SharedArray0[GroupThreadIndex] = HalfResOutput;
			HalfResOutput += SharedArray0[GroupThreadIndex ^ 0x1];
			
			SharedArray0[GroupThreadIndex] = HalfResOutput;
			HalfResOutput += SharedArray0[GroupThreadIndex ^ 0x2];
		}
		#endif
		else
		{
			HalfResOutput = 0.0;
		}
		
		// Needed to avoid crash in shader compiler of Xclipse driver
		#if !CONFIG_SCENE_COLOR_ALPHA
			HalfResOutput.a = 0;
		#endif
		mb_half4 QuarterResOutput;
		BRANCH
		if (OutputMip2)
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST
		{
			FWaveBroadcastSettings Horizontal = InitWaveXorButterfly(/* XorButterFly = */ 0x4);
			FWaveBroadcastSettings Vertical = InitWaveXorButterfly(/* XorButterFly = */ 0x8);
			
			QuarterResOutput = HalfResOutput * mb_half(0.25);
			QuarterResOutput += WaveBroadcast(Horizontal, QuarterResOutput);
			QuarterResOutput += WaveBroadcast(Vertical, QuarterResOutput);
		}
		#else
		{
			QuarterResOutput = HalfResOutput * mb_half(0.25);

			SharedArray0[GroupThreadIndex] = QuarterResOutput;
			QuarterResOutput += SharedArray0[GroupThreadIndex ^ 0x4];
			
			SharedArray0[GroupThreadIndex] = QuarterResOutput;
			QuarterResOutput += SharedArray0[GroupThreadIndex ^ 0x8];
		}
		#endif
		else
		{
			QuarterResOutput = 0.0;
		}

		// Needed to avoid crash in shader compiler of Xclipse driver
		#if !CONFIG_SCENE_COLOR_ALPHA
			QuarterResOutput.a = 0;
		#endif
		bool bIsValid = all(OutputPixelCoord < mb_short2(Color_ViewportMax));
		mb_short2 OutputPixelCoordMip0 = (bIsValid) ? OutputPixelCoord : mb_short(~0).xx;
		mb_short2 OutputPixelCoordMip1 = select(and(bIsValid, and(OutputMip1 != 0, (OutputPixelCoordMip0 & 0x1) == 0)), (OutputPixelCoord >> mb_short(1)), mb_short(~0).xx);
		mb_short2 OutputPixelCoordMip2 = select(and(bIsValid, and(OutputMip2 != 0, (OutputPixelCoordMip0 & 0x3) == 0)), (OutputPixelCoord >> mb_short(2)), mb_short(~0).xx);

		SceneColorOutputMip0[OutputPixelCoordMip0] = OutputColor;
		SceneColorOutputMip1[OutputPixelCoordMip1] = HalfResOutput;
		SceneColorOutputMip2[OutputPixelCoordMip2] = QuarterResOutput;

		#if DEBUG_MOTION_BLUR_OUTPUT
			DebugOutput[OutputPixelCoordMip0] = Debug;
		#endif
	}
}
