// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../FastMath.ush"
#include "../ScreenPass.ush"


//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


//------------------------------------------------------- CONFIG

#define DEBUG_MOTION_BLUR_OUTPUT 0

#ifndef CONFIG_MAX_RANGE_SIZE
	#error need to set CONFIG_MAX_RANGE_SIZE
#endif

// Size in input pixel of the velocity flatten tiling
#define VELOCITY_FLATTEN_TILE_SIZE 16

// Size of the tile at full resolution
#define VELOCITY_FILTER_TILE_SIZE 16

#if PLATFORM_SUPPORTS_REAL_TYPES && 0
	#define CONFIG_MOTION_BLUR_COMPILE_FP16 1
#else
	#define CONFIG_MOTION_BLUR_COMPILE_FP16 0
#endif

#define CONFIG_MINIMAL_PIXEL_VELOCITY 0.5


//------------------------------------------------------- CONSTANTS

#if CONFIG_MOTION_BLUR_COMPILE_FP16
	#define mb_half  half
	#define mb_half2 half2
	#define mb_half3 half3
	#define mb_half4 half4

	#define mb_short  int16_t
	#define mb_short2 int16_t2
	#define mb_short3 int16_t3
	#define mb_short4 int16_t4

	#define mb_ushort  uint16_t
	#define mb_ushort2 uint16_t2
	#define mb_ushort3 uint16_t3
	#define mb_ushort4 uint16_t4

	#define mb_half2x2 half2x2
	#define mb_half3x2 half3x2
	#define mb_half4x2 half4x2

	#define mb_short2x2 int16_t2x2
	#define mb_short3x2 int16_t3x2
	#define mb_short4x2 int16_t4x2

	#define mb_ushort2x2 uint16_t2x2
	#define mb_ushort3x2 uint16_t3x2
	#define mb_ushort4x2 uint16_t4x2

#else
	#define mb_half  float
	#define mb_half2 float2
	#define mb_half3 float3
	#define mb_half4 float4

	#define mb_short  int
	#define mb_short2 int2
	#define mb_short3 int3
	#define mb_short4 int4

	#define mb_ushort  uint
	#define mb_ushort2 uint2
	#define mb_ushort3 uint3
	#define mb_ushort4 uint4

	#define mb_half2x2 float2x2
	#define mb_half3x2 float3x2
	#define mb_half4x2 float4x2

	#define mb_short2x2 int2x2
	#define mb_short3x2 int3x2
	#define mb_short4x2 int4x2

	#define mb_ushort2x2 uint2x2
	#define mb_ushort3x2 uint3x2
	#define mb_ushort4x2 uint4x2

#endif

//------------------------------------------------------- DEFINES

/** Filter tile classification. Matches FMotionBlurFilterCS::ETileClassification */
#define FILTER_TILE_CLASSIFY_GATHER_HALF_RES 0
#define FILTER_TILE_CLASSIFY_GATHER_FULL_RES 1
#define FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_HALF_RES 2
#define FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES 3
#define FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_2_VELOCITY_FULL_RES 4
#define FILTER_TILE_CLASSIFY_COUNT 5



//------------------------------------------------------- FUNCTIONS

float4 MinMaxLength(float4 v0, float4 v1)
{
	float2 Min = dot(v0.xy, v0.xy) < dot(v1.xy, v1.xy) ? v0.xy : v1.xy;
	float2 Max = dot(v0.zw, v0.zw) > dot(v1.zw, v1.zw) ? v0.zw : v1.zw;
	return float4(Min, Max);
}

float4 MinMaxLengthPolar(float4 v0, float4 v1)
{
	float2 Min = v0.x < v1.x ? v0.xy : v1.xy;
	float2 Max = v0.z > v1.z ? v0.zw : v1.zw;
	return float4(Min, Max);
}

float2 CartesianToPolar(float2 Velocity)
{
	float Length = length(Velocity);
	float Angle = Length > 0.0 ? atan2Fast(Velocity.y, Velocity.x) : 0.0;
	return float2(Length, Angle);
}

float2 PolarToCartesian(float2 Velocity)
{
	float Length = Velocity.x;
	float Angle = Velocity.y;
	sincos(Angle, Velocity.y, Velocity.x);
	return Velocity * Length;
}

float GetPolarRelativeAngle(float V0, float V1)
{
	float AbsDiff = abs(V0 - V1);

	if (AbsDiff > PI)
	{
		AbsDiff = 2.0 * PI - AbsDiff;
	}

	return AbsDiff;
}

float2 GetMaxPolarvelocity(float2 V0, float2 V1)
{
	return V0.x > V1.x ? V0 : V1;
}

float2 GetMinPolarvelocity(float2 V0, float2 V1)
{
	return V0.x > V1.x ? V1 : V0;
}

float2 GetMaxPolarvelocity(float2 V0, float2 V1, float2 V2)
{
	return GetMaxPolarvelocity(GetMaxPolarvelocity(V0, V1), V2);
}


//------------------------------------------------------- TILED VELOCITY RANGE FUNCTIONS

struct FVelocityRange
{
	float2 Max[CONFIG_MAX_RANGE_SIZE];
	float2 Min;
};

FVelocityRange SetupPolarVelocityRange(float2 PolarVelocity)
{
	FVelocityRange V;
	V.Max[0] = PolarVelocity;

	#if CONFIG_MAX_RANGE_SIZE > 1
		V.Max[1] = 0.0;
	#endif

	V.Min = PolarVelocity;
	return V;
}

FVelocityRange ReducePolarVelocityRange(FVelocityRange V0, FVelocityRange V1)
{
	FVelocityRange V;
	V.Max[0] = GetMaxPolarvelocity(V0.Max[0], V1.Max[0]);
	
	#if CONFIG_MAX_RANGE_SIZE > 1
	if (GetPolarRelativeAngle(V0.Max[0].y, V1.Max[0].y) < ((PI / 180) * 5.0) || 0)
	{
		V.Max[1] = GetMaxPolarvelocity(V0.Max[1], V1.Max[1]);
	}
	else
	{
		V.Max[1] = GetMaxPolarvelocity(GetMinPolarvelocity(V0.Max[0], V1.Max[0]), V0.Max[1], V1.Max[1]);
	}
	#endif
	
	V.Min = GetMinPolarvelocity(V0.Min, V1.Min);

	return V;
}

FVelocityRange CartesianToPolar(FVelocityRange CartesianRange)
{
	FVelocityRange PolarRange;
	PolarRange.Max[0] = CartesianToPolar(CartesianRange.Max[0]);
	#if CONFIG_MAX_RANGE_SIZE > 1
		PolarRange.Max[1] = CartesianToPolar(CartesianRange.Max[1]);
	#endif
	PolarRange.Min = CartesianToPolar(CartesianRange.Min);
	return PolarRange;
}

FVelocityRange PolarToCartesian(FVelocityRange PolarRange)
{
	FVelocityRange CartesianRange;
	CartesianRange.Max[0] = PolarToCartesian(PolarRange.Max[0]);
	#if CONFIG_MAX_RANGE_SIZE > 1
		CartesianRange.Max[1] = PolarToCartesian(PolarRange.Max[1]);
	#endif
	CartesianRange.Min = PolarToCartesian(PolarRange.Min);
	return CartesianRange;
}

void StoreVelocityRange(
	RWTexture2DArray<float4> VelocityFlatTextureArray,
	uint2 PixelCoord,
	FVelocityRange CartesianRange)
{
	VelocityFlatTextureArray[uint3(PixelCoord, 0)] = float4(CartesianRange.Min, CartesianRange.Max[0]);
	#if CONFIG_MAX_RANGE_SIZE > 1
		VelocityFlatTextureArray[uint3(PixelCoord, 1)] = float4(CartesianRange.Max[1], 0.0, 0.0);
	#endif
}

FVelocityRange DecodeVelocityRange(
	float4 Raw0,
	float4 Raw1)
{
	FVelocityRange CartesianRange;
	CartesianRange.Max[0] = Raw0.zw;
	#if CONFIG_MAX_RANGE_SIZE > 1
		CartesianRange.Max[1] = Raw1.xy;
	#endif
	CartesianRange.Min = Raw0.xy;
	return CartesianRange;
}

FVelocityRange LoadVelocityRange(
	Texture2D<float4> VelocityFlatTexture_0,
	Texture2D<float4> VelocityTileTexture_1,
	uint2 PixelCoord)
{
	float4 Raw0 = VelocityFlatTexture_0[PixelCoord];
	float4 Raw1 = VelocityTileTexture_1[PixelCoord];

	return DecodeVelocityRange(Raw0, Raw1);
}
