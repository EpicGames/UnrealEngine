// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE DIM_BLUR_DIRECTIONS


#include "MotionBlurCommon.ush"

//------------------------------------------------------- CONFIG

#define TILE_SIZE 16


//------------------------------------------------------- PARAMETERS

FScreenTransform FilterTileIdToFlattenTileId;
uint2 FlattenTileMaxId;
uint2 FilterTileCount;
uint bAllowHalfResGather;
uint TileListMaxSize;
float HalfResPixelVelocityThreshold;

Texture2D VelocityTileTextures_Textures_0;
Texture2D VelocityTileTextures_Textures_1;

RWCoherentStructuredBuffer(uint) TileListsOutput;
RWCoherentStructuredBuffer(uint) TileListsSizeOutput;

RWTexture2D<float4> DebugOutput;


//------------------------------------------------------- LDS

groupshared uint SharedTileCount[FILTER_TILE_CLASSIFY_COUNT];


//------------------------------------------------------- ENTRY POINT

#ifdef SETUP_PASS

StructuredBuffer<uint> TileListsSizeBuffer;
RWBuffer<uint> DispatchParametersOutput;

[numthreads(FILTER_TILE_CLASSIFY_COUNT, 1, 1)]
void MainCS(uint GroupThreadIndex : SV_GroupIndex)
{
	if (GroupThreadIndex < FILTER_TILE_CLASSIFY_COUNT)
	{
		WriteDispatchIndirectArgs(DispatchParametersOutput, GroupThreadIndex, TileListsSizeBuffer[GroupThreadIndex], 1, 1);
	}
}

#else // !SETUP_PASS

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Debug = 0;
	
	uint2 DispatchThreadId = (
		ZOrder2D(GroupThreadIndex, log2(TILE_SIZE)) +
		GroupId * uint2(TILE_SIZE, TILE_SIZE));

	mb_short2 FilterTileCoord = DispatchThreadId;
	bool bIsValidTileCoord = all(FilterTileCoord < mb_short2(FilterTileCount));

	mb_short2 FlattenTileCoord = mb_short2(ApplyScreenTransform(mb_half2(FilterTileCoord), FilterTileIdToFlattenTileId));

	uint TileClassify = FILTER_TILE_CLASSIFY_GATHER_HALF_RES;

	UNROLL_N(3)
	for (int y = -1; y <= 1; y++)
	{
		UNROLL_N(3)
		for (int x = -1; x <= 1; x++)
		{
			const mb_short2 Offset = mb_short2(x, y);

			mb_short2 NeighborFlattenTileCoord = FlattenTileCoord + Offset;
			NeighborFlattenTileCoord = clamp(NeighborFlattenTileCoord, mb_short(0).xx, mb_short2(FlattenTileMaxId));

			FVelocityRange VelocityRange = DecodeVelocityRange(
				VelocityTileTextures_Textures_0[NeighborFlattenTileCoord],
				VelocityTileTextures_Textures_1[NeighborFlattenTileCoord]);
			
			float MinVelocityLengthSqrPixels = dot(VelocityRange.Min, VelocityRange.Min);
			float Max0VelocityLengthSqrPixels = dot(VelocityRange.Max[0], VelocityRange.Max[0]);
			
			bool bHalfRes = (Max0VelocityLengthSqrPixels > HalfResPixelVelocityThreshold) && (bAllowHalfResGather != 0);

			#if CONFIG_MAX_RANGE_SIZE > 1
				float Max1VelocityLengthSqrPixels = dot(VelocityRange.Max[1], VelocityRange.Max[1]);
				bool bDoOneDirectionOnly = Max1VelocityLengthSqrPixels < (CONFIG_MINIMAL_PIXEL_VELOCITY * CONFIG_MINIMAL_PIXEL_VELOCITY);
			#else
				const bool bDoOneDirectionOnly = true;
			#endif

			bool bSkip0 = Max0VelocityLengthSqrPixels < (CONFIG_MINIMAL_PIXEL_VELOCITY * CONFIG_MINIMAL_PIXEL_VELOCITY);
			bool bGatherOnly = MinVelocityLengthSqrPixels > 0.4 * Max0VelocityLengthSqrPixels;

			uint SuggestedTileClassification = FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES;
			if (bSkip0)
			{
				// Dissregard bAllowHalfRes because FILTER_TILE_CLASSIFY_GATHER_HALF_RES is still the quickest when no motion blur is to applied.
				SuggestedTileClassification = FILTER_TILE_CLASSIFY_GATHER_HALF_RES;
			}
			#if CONFIG_MAX_RANGE_SIZE > 1 && 1
			else if (!bDoOneDirectionOnly)
			{
				SuggestedTileClassification = FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_2_VELOCITY_FULL_RES;
			}
			#endif
			else if (bGatherOnly)
			{
				SuggestedTileClassification = bHalfRes ? FILTER_TILE_CLASSIFY_GATHER_HALF_RES : FILTER_TILE_CLASSIFY_GATHER_FULL_RES;
			}
			else
			{
				SuggestedTileClassification = bHalfRes ? FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_HALF_RES : FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES;
			}

			TileClassify = max(TileClassify, SuggestedTileClassification);
		}
	}

	if (GroupThreadIndex < FILTER_TILE_CLASSIFY_COUNT)
	{
		SharedTileCount[GroupThreadIndex] = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();
	
	// Alloc within the group
	uint LocalAllocation = 0;
	if (bIsValidTileCoord)
	{
		InterlockedAdd(SharedTileCount[TileClassify], 1, /* out */ LocalAllocation);
	}
	
	GroupMemoryBarrierWithGroupSync();
	
	// Alloc globally
	if (GroupThreadIndex < FILTER_TILE_CLASSIFY_COUNT)
	{
		uint GlobalGroupAllocation;
		InterlockedAdd(TileListsSizeOutput[GroupThreadIndex], SharedTileCount[GroupThreadIndex], /* out */ GlobalGroupAllocation);

		SharedTileCount[GroupThreadIndex] = GlobalGroupAllocation;
	}
	
	GroupMemoryBarrierWithGroupSync();
	
	// Output
	uint GlobalAllocation = SharedTileCount[TileClassify] + LocalAllocation;

	if (TileClassify == FILTER_TILE_CLASSIFY_GATHER_HALF_RES)
	{
		Debug = float4(0, 1, 0, 0);
	}
	else if (TileClassify == FILTER_TILE_CLASSIFY_GATHER_FULL_RES)
	{
		Debug = float4(0, 0, 1, 0);
	}
	else if (TileClassify == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_HALF_RES || TileClassify == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_1_VELOCITY_FULL_RES)
	{
		Debug = float4(1, 1, 0, 0);
	}
	else if (TileClassify == FILTER_TILE_CLASSIFY_SCATTER_AS_GATHER_2_VELOCITY_FULL_RES)
	{
		Debug = float4(1, 0, 0, 0);
	}


	if (bIsValidTileCoord)
	{
		TileListsOutput[GlobalAllocation + TileListMaxSize * TileClassify] = uint(FilterTileCoord.x) | (uint(FilterTileCoord.y) << 16);

		#if DEBUG_MOTION_BLUR_OUTPUT
			DebugOutput[FilterTileCoord] = Debug;
		#endif
	}
}

#endif // !SETUP_PASS
