// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE DIM_BLUR_DIRECTIONS

#include "MotionBlurCommon.ush"


//------------------------------------------------------- PARAMETERS

SCREEN_PASS_TEXTURE_VIEWPORT(VelocityTile)
float VelocityScaleForFlattenTiles;

Texture2D CenterVelocityTileTextures_Textures_0;
Texture2D CenterVelocityTileTextures_Textures_1;

Texture2D VelocityTileTextures_Textures_0;
Texture2D VelocityTileTextures_Textures_1;

RWTexture2DArray<float4>	OutVelocityTileArray;


//------------------------------------------------------- ENTRY POINT

[numthreads(16, 16, 1)]
void MainCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex : SV_GroupIndex)
{
	uint2 PixelPos = DispatchThreadId.xy;
	float2 UV = ((float2)PixelPos.xy + 0.5) * VelocityTile_ExtentInverse;
	
	FVelocityRange VelocityPolarRange = CartesianToPolar(LoadVelocityRange(CenterVelocityTileTextures_Textures_0, CenterVelocityTileTextures_Textures_1, PixelPos));

	// Scatter as gather
	for(int x = -3; x <= 3; x++)
	{
		for(int y = -3; y <= 3; y++)
		{
			if (x == 0 && y == 0)
				continue;

			int2 Offset = int2(x,y);
			int2 SampleIndex = PixelPos + Offset;
			
			bool2 bInsideViewport = and(0 <= SampleIndex, SampleIndex < (int2)VelocityTile_ViewportMax);
			if (!all(bInsideViewport))
				continue;
			
			FVelocityRange NeighborVelocityRange = LoadVelocityRange(VelocityTileTextures_Textures_0, VelocityTileTextures_Textures_1, SampleIndex);

			float2 MaxVelocity = NeighborVelocityRange.Max[0];

			float2 VelocityPixels = MaxVelocity * VelocityScaleForFlattenTiles;

			float  VelocityLengthPixelsSqr = dot(VelocityPixels, VelocityPixels);
			float  VelocityLengthPixelsInv = rsqrtFast(VelocityLengthPixelsSqr + 1e-8);
			float  VelocityLengthPixels = VelocityLengthPixelsSqr * VelocityLengthPixelsInv;
			float2 VelocityDir = VelocityPixels * VelocityLengthPixelsInv;

			// Project pixel corner on to dir. This is the oriented extent of a pixel.
			// 1/2 pixel because shape is swept tile
			// +1/2 pixel for conservative rasterization
			// 99% to give epsilon before neighbor is filled. Otherwise all neighbors lie on edges of quad when no velocity in their direction.
			float PixelExtent = abs(VelocityDir.x) + abs(VelocityDir.y);
			float2 QuadExtent = float2(VelocityLengthPixels, 0) + PixelExtent.xx * 0.99;

			// Orient quad along velocity direction
			float2 AxisX = VelocityDir;
			float2 AxisY = float2(-VelocityDir.y, VelocityDir.x);

			// Project this pixel center onto scatter quad
			float2 PixelCenterOnQuad;
			PixelCenterOnQuad.x = dot(AxisX, Offset);
			PixelCenterOnQuad.y = dot(AxisY, Offset);

			bool2 bInsideQuad = abs(PixelCenterOnQuad) < QuadExtent;
			if (all(bInsideQuad)) 
			{
				VelocityPolarRange = ReducePolarVelocityRange(VelocityPolarRange, CartesianToPolar(NeighborVelocityRange));
			}
		}
	}

	BRANCH
	if (all(PixelPos.xy < VelocityTile_ViewportMax))
	{
		StoreVelocityRange(OutVelocityTileArray, PixelPos, PolarToCartesian(VelocityPolarRange));
	}
}
