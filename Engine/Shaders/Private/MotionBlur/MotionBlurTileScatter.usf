// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE 1


#include "MotionBlurCommon.ush"

//------------------------------------------------------- CONFIG

#define VELOCITY_SCATTER_PASS_MIN 0
#define VELOCITY_SCATTER_PASS_MAX 1


//------------------------------------------------------- PARAMETERS

SCREEN_PASS_TEXTURE_VIEWPORT(VelocityTile)
float VelocityScaleForFlattenTiles;
uint ScatterPass;

Texture2D VelocityTileTextures_Textures_0;
Texture2D VelocityTileTextures_Textures_1;


//------------------------------------------------------- ENTRY POINT

void MainVS(
	uint VId : SV_VertexID,
	uint IId : SV_InstanceID,
	out nointerpolation float4 OutColor : TEXCOORD0,
	out float4 OutPosition : SV_POSITION)
{
	OutPosition = float4(0, 0, 0, 1);

	// needs to be the same on C++ side (faster on NVIDIA and AMD)
	uint QuadsPerInstance = 8;

	// remap the indices to get vertexid to VId and quadid into IId
	IId = IId * QuadsPerInstance + (VId / 4);
	VId = VId % 4;

	// triangle A: 0:left top, 1:right top, 2: left bottom
	// triangle B: 3:right bottom, 4:left bottom, 5: right top
	float2 CornerOffset = float2(VId % 2, VId / 2) * 2 - 1;

	uint2 PixelPos = uint2(IId % VelocityTile_ViewportMax.x, IId / VelocityTile_ViewportMax.x);

	BRANCH
	if (PixelPos.y >= VelocityTile_ViewportMax.y)
	{
		OutColor = 0;
		return;
	}

	float2 SvPosition = PixelPos + 0.5;

	float4 MinMaxVelocity = VelocityTileTextures_Textures_0[PixelPos];
	OutColor = MinMaxVelocity;

	float4 MinMaxVelocityPixels = MinMaxVelocity * VelocityScaleForFlattenTiles;
	float2 VelocityPixels = MinMaxVelocityPixels.zw;

	// Is the velocity small enough not to cover adjacent tiles?
	BRANCH
	if (dot(VelocityPixels, VelocityPixels) * 16 * 16 <= 0.25)
	{
		OutPosition.xy = (SvPosition + CornerOffset * 0.5 - VelocityTile_ScreenPosToViewportBias) / VelocityTile_ScreenPosToViewportScale.xy;
		OutPosition.z = 0.0002; // zero clips
		return;
	}

	float  VelocityLengthPixelsSqr = dot(VelocityPixels, VelocityPixels);
	float  VelocityLengthPixelsInv = rsqrtFast(VelocityLengthPixelsSqr);
	float  VelocityLengthPixels = VelocityLengthPixelsSqr * VelocityLengthPixelsInv;
	float2 VelocityDir = VelocityPixels * VelocityLengthPixelsInv;

	// Project pixel corner on to dir. This is the oriented extent of a pixel.
	// 1/2 pixel because shape is swept tile
	// +1/2 pixel for conservative rasterization
	// 99% to give epsilon before neighbor is filled. Otherwise all neighbors lie on edges of quad when no velocity in their direction.
	// dot(abs(VelocityDir), float2(1, 1))
	float Extent = abs(VelocityDir.x) + abs(VelocityDir.y);
	CornerOffset *= float2(VelocityLengthPixels, 0) + Extent.xx * 0.99;

	// Orient along velocity direction
	float2 AxisX = VelocityDir;
	float2 AxisY = float2(-VelocityDir.y, VelocityDir.x);
	CornerOffset = AxisX * CornerOffset.x + AxisY * CornerOffset.y;

	OutPosition.xy = (SvPosition + CornerOffset - VelocityTile_ScreenPosToViewportBias) / VelocityTile_ScreenPosToViewportScale;

	// Depth ordered by velocity length
	OutPosition.z = (ScatterPass == VELOCITY_SCATTER_PASS_MAX) ? VelocityLengthPixels : length(MinMaxVelocityPixels.xy);
	OutPosition.z = clamp(OutPosition.z / VelocityTile_ScreenPosToViewportScale.x * 0.5, 0.0002, 0.999);
}

void MainPS(
	nointerpolation float4 InColor : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	OutColor = InColor;
}
