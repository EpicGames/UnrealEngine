// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE DIM_BLUR_DIRECTIONS

#include "MotionBlurVelocityFlatten.ush"
#include "../LensDistortion.ush"


//------------------------------------------------------- CONSTANTS

#define TILE_SIZE				(VELOCITY_FLATTEN_TILE_SIZE)
#define THREADGROUP_TOTALSIZE	(TILE_SIZE * TILE_SIZE)

#ifndef CAMERA_MOTION_BLUR_MODE
// 0-off; 1-on; 2-override
#define CAMERA_MOTION_BLUR_MODE 1
#endif


//------------------------------------------------------- PARAMETERS

Texture2D<float2> DistortingDisplacementTexture;
SamplerState      DistortingDisplacementSampler;

Texture2D<float2> UndistortingDisplacementTexture;
SamplerState      UndistortingDisplacementSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Velocity)

float4x4 ClipToPrevClipOverride;
FScreenTransform ThreadIdToViewportUV;
FScreenTransform ViewportUVToPixelPos;
uint bCancelCameraMotion;
uint bAddCustomCameraMotion;
uint bLensDistortion;

Texture2D VelocityTexture;
Texture2D DepthTexture;

RWTexture2D<float3>	OutVelocityFlatTexture;
RWTexture2DArray<float4>	OutVelocityTileArray;

RWTexture2D<float4>	DebugOutput;


//------------------------------------------------------- FUNCTIONS

float2 GetCameraMotionVelocity(uint2 DispatchThreadId, float Depth, float4x4 ClipToPrevClip)
{
	// Compute velocity due to camera motion.
	float2 ViewportUV = ((float2)DispatchThreadId.xy + 0.5) / Velocity_ViewportSize;
	float2 ScreenPos = 2 * float2(ViewportUV.x, 1 - ViewportUV.y) - 1;
	float4 ThisClip = float4(ScreenPos, Depth, 1);
	float4 PrevClip = mul(ThisClip, ClipToPrevClip);
	float2 PrevScreen = PrevClip.xy / PrevClip.w;
	return AdjustClipToPrevClipForProjectionType(ScreenPos - PrevScreen, Depth);
}

float2 PreprocessVelocityForMotionBlur(uint2 DispatchThreadId, float DeviceZ, float2 Velocity, float2 CameraMotionVelocity)
{
	FLATTEN
	if (bCancelCameraMotion)
	{
		Velocity -= CameraMotionVelocity;
	}

	BRANCH
	if (bAddCustomCameraMotion)
	{
		Velocity += GetCameraMotionVelocity(DispatchThreadId, DeviceZ, ClipToPrevClipOverride);
	}

	return Velocity;
}


//------------------------------------------------------- ENTRY POINT

[numthreads(VELOCITY_FLATTEN_TILE_SIZE, VELOCITY_FLATTEN_TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex : SV_GroupIndex) 
{ 
	float4 Debug = 0;

	int2 OutputPixelPos = min(DispatchThreadId.xy + Velocity_ViewportMin, Velocity_ViewportMax - 1);
	int2 PixelPos = OutputPixelPos;

	BRANCH
	if (bLensDistortion)
	{
		float2 DistortedViewportUV = ApplyScreenTransform(float2(DispatchThreadId), ThreadIdToViewportUV);
		float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);
		PixelPos = int2(ApplyScreenTransform(UndistortedViewportUV, ViewportUVToPixelPos));
	}

	int2 DepthOffset = 0;

	{
		const int Cross = 1;
		// For motion vector, use camera/dynamic motion from min depth pixel in pattern around pixel.
		// This enables better quality outline on foreground against different motion background.
		
		half4 Depths;
		Depths.x = DepthTexture[ PixelPos + int2(-Cross, -Cross) ].x;
		Depths.y = DepthTexture[ PixelPos + int2( Cross, -Cross) ].x;
		Depths.z = DepthTexture[ PixelPos + int2(-Cross,  Cross) ].x;
		Depths.w = DepthTexture[ PixelPos + int2( Cross,  Cross) ].x;

		DepthOffset = Cross;
		int DepthOffsetXx = Cross;

		// Nearest depth is the largest depth (depth surface 0=far, 1=near).
		if(Depths.x > Depths.y) 
		{
			DepthOffsetXx = -Cross;
		}
		if(Depths.z > Depths.w) 
		{
			DepthOffset.x = -Cross;
		}
		half DepthsXY = max(Depths.x, Depths.y);
		half DepthsZW = max(Depths.z, Depths.w);
		if(DepthsXY > DepthsZW) 
		{
			DepthOffset.y = -Cross;
			DepthOffset.x = DepthOffsetXx; 
		}
	}

	float4 EncodedVelocity = VelocityTexture[ PixelPos + DepthOffset ];
	float DeviceZ = DepthTexture[ PixelPos + DepthOffset ].x;

	float2 CameraMotionVelocity = GetCameraMotionVelocity(DispatchThreadId, DeviceZ, View.ClipToPrevClip);
	float2 Velocity = CameraMotionVelocity;
	FLATTEN
	if (EncodedVelocity.x > 0.0)
	{
		Velocity = DecodeVelocityFromTexture(EncodedVelocity).xy;
	}

	Velocity = PreprocessVelocityForMotionBlur(DispatchThreadId, DeviceZ, Velocity, CameraMotionVelocity);

	FVelocityRange VelocityPolarRange;
	float2 VelocityPolar;
	ReduceVelocityFlattenTile(GroupIndex, Velocity, /* out */ VelocityPolarRange, /* out */ VelocityPolar);

	// Output velcity flatten texture.
	{
		uint2 OutputPos = select(OutputPixelPos.xy < Velocity_ViewportMax, uint2(OutputPixelPos), uint(~0).xx);
		OutVelocityFlatTexture[OutputPos] = EncodeVelocityFlatten(VelocityPolar, DeviceZ);
	}

	// Output tile
	{
		uint2 OutputPos = GroupIndex == 0 ? GroupId : uint(~0).xx;
		StoreVelocityRange(OutVelocityTileArray, OutputPos, PolarToCartesian(VelocityPolarRange));
	}
	
#if DEBUG_MOTION_BLUR_OUTPUT
	DebugOutput[OutputPixelPos] = Debug;
#endif
}
