// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionBlurCommon.ush"


//------------------------------------------------------- CONSTANTS

#define VELOCITY_FLATTEN_THREAD_COUNT (VELOCITY_FLATTEN_TILE_SIZE * VELOCITY_FLATTEN_TILE_SIZE)


//------------------------------------------------------- PARAMETERS

float2 VelocityScale;
float VelocityMax;


//------------------------------------------------------- LDS

groupshared float4 SharedArray0[VELOCITY_FLATTEN_THREAD_COUNT];
groupshared float2 SharedArray1[VELOCITY_FLATTEN_THREAD_COUNT];


//------------------------------------------------------- FUNCTIONS

void WritePolarVelocityRangeToLDS(uint LDSSlotId, FVelocityRange V)
{
	SharedArray0[LDSSlotId] = float4(V.Max[0], V.Min);
	
	#if CONFIG_MAX_RANGE_SIZE > 1
		SharedArray1[LDSSlotId] = V.Max[1];
	#endif
}

FVelocityRange ReadPolarVelocityRangeFromLDS(uint LDSSlotId)
{
	FVelocityRange V;
	V.Max[0] = SharedArray0[LDSSlotId].xy;
	#if CONFIG_MAX_RANGE_SIZE > 1
		V.Max[1] = SharedArray1[LDSSlotId].xy;
	#endif
	V.Min = SharedArray0[LDSSlotId].zw;
	return V;
}

void VelocityFlattenStep(uint GroupIndex, const uint GroupSize, inout FVelocityRange V)
{
	if (GroupIndex < GroupSize)
	{
		FVelocityRange V1 = ReadPolarVelocityRangeFromLDS(GroupIndex + GroupSize);
		
		//if (GroupSize == uint(View.GeneralPurposeTweak))
		//{
		//	Debug[GroupIndex].r = V.Max[1].g;
		//}

		V = ReducePolarVelocityRange(V, V1);
		
		//if (GroupSize == uint(View.GeneralPurposeTweak))
		//{
		//	Debug[GroupIndex].g = V.Max[1].g;
		//}

		WritePolarVelocityRangeToLDS(GroupIndex, V);
	}
}

void ReduceVelocityFlattenTile(uint GroupIndex, float2 Velocity, out FVelocityRange OutVelocityPolarRange, out float2 VelocityPolar)
{
	Velocity *= VelocityScale;

	VelocityPolar = CartesianToPolar(Velocity);

	// If the velocity vector was zero length, VelocityPolar will contain NaNs.
	if (any(isnan(VelocityPolar)))
	{
		VelocityPolar = float2(0.0f, 0.0f);
	}

	// Limit velocity
	VelocityPolar.x = min(VelocityPolar.x, VelocityMax);

	FVelocityRange VelocityPolarRange = SetupPolarVelocityRange(VelocityPolar);

	WritePolarVelocityRangeToLDS(GroupIndex, VelocityPolarRange);

	GroupMemoryBarrierWithGroupSync();

#if VELOCITY_FLATTEN_THREAD_COUNT > 512
	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 512, /* inout */ VelocityPolarRange);
	GroupMemoryBarrierWithGroupSync();
#endif
#if VELOCITY_FLATTEN_THREAD_COUNT > 256
	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 256, /* inout */ VelocityPolarRange);
	GroupMemoryBarrierWithGroupSync();
#endif
#if VELOCITY_FLATTEN_THREAD_COUNT > 128
	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 128, /* inout */ VelocityPolarRange);
	GroupMemoryBarrierWithGroupSync();
#endif
#if VELOCITY_FLATTEN_THREAD_COUNT > 64
	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 64, /* inout */ VelocityPolarRange);
	GroupMemoryBarrierWithGroupSync();
#endif

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	const uint LaneCount = WaveGetLaneCount();
#else
	// Actual wave size is unknown, assume the worst
	const uint LaneCount = 0u;
#endif

	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 32, /* inout */ VelocityPolarRange);
	if (LaneCount < 32)
	{
		GroupMemoryBarrierWithGroupSync();
	}
	
	VelocityFlattenStep(GroupIndex, /* GroupSize = */ 16, /* inout */ VelocityPolarRange);
	if (LaneCount < 16)
	{
		GroupMemoryBarrierWithGroupSync();
	}
	
	VelocityFlattenStep(GroupIndex, /* GroupSize = */  8, /* inout */ VelocityPolarRange);
	if (LaneCount < 8)
	{
		GroupMemoryBarrierWithGroupSync();
	}
	
	VelocityFlattenStep(GroupIndex, /* GroupSize = */  4, /* inout */ VelocityPolarRange);	
	VelocityFlattenStep(GroupIndex, /* GroupSize = */  2, /* inout */ VelocityPolarRange);
	VelocityFlattenStep(GroupIndex, /* GroupSize = */  1, /* inout */ VelocityPolarRange);

	OutVelocityPolarRange = VelocityPolarRange;
}

float3 EncodeVelocityFlatten(float2 VelocityPolar, float DeviceZ)
{
	// 11:11:10  (VelocityLength, VelocityAngle, Depth)
	float2 EncodedPolarVelocity;
	EncodedPolarVelocity.x = VelocityPolar.x;
	EncodedPolarVelocity.y = VelocityPolar.y * (0.5 / PI) + 0.5;

	return float3(EncodedPolarVelocity, ConvertFromDeviceZ(DeviceZ)).xyz;
}
