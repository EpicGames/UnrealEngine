// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_MAX_RANGE_SIZE 1

#include "MotionBlurCommon.ush"
#include "../LensDistortion.ush"
#include "../ColorMap.ush"

//------------------------------------------------------- CONSTANTS

// 0:off / 1:on, useful to debug the motionblur algorithm
#define MOTIONBLUR_TESTCHART 0

#define VISUALIZE_COLOR_ENCODING 1
#define TILE_SIZE 48.0f
#define COLOR_METER_LENGTH TILE_SIZE
#define COLOR_METER_OFFSET 5

//------------------------------------------------------- PARAMETERS

Texture2D<float2> UndistortingDisplacementTexture;
SamplerState      UndistortingDisplacementSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Velocity)

Texture2D VelocityTexture;
Texture2D DepthTexture;

SamplerState VelocitySampler;
SamplerState DepthSampler;

FScreenTransform SvPositionToScreenPos;
FScreenTransform ScreenPosToColorUV;
FScreenTransform ScreenPosToVelocityUV;
int CheckerboardEnabled;
int VisualizeMode;

#if USE_POST_MOTION_BLUR_TRANSLUCENCY

Texture2D PostMotionBlurTranslucencyTexture;
SamplerState PostMotionBlurTranslucencySampler;

#endif // USE_POST_MOTION_BLUR_TRANSLUCENCY

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedVelocitySampler VelocitySampler
	#define SharedDepthSampler VelocitySampler
#else
	#define SharedVelocitySampler VelocitySampler
	#define SharedDepthSampler DepthSampler
#endif


//------------------------------------------------------- FUNCTIONS

// debug motionblur (very useful, keep)
// @param ScreenPos -1..1 -1..1 for viewport
// @param Velocity in -1..1 range for full motionblur
// @apram Color RGB and depth in alpha
// @param AvgObject 0:background, 1:foregound
void OverrideWithTestChart(float2 ScreenPos, inout float2 ObjectVelocity, inout float2 BackgroundVelocity, inout float4 Color, inout float AvgObject)
{
#if MOTIONBLUR_TESTCHART == 1
	// needs to be inside the loop to prevent NVIDIA driver optimizetion (blinking)
	float2 PixelPos = ScreenPos * Velocity_ScreenPosToViewportScale.xy + Velocity_ScreenPosToViewportBias - 25;
	float3 BackgroundColor = lerp(0.0, 0.3f, PseudoRandom(PixelPos)); 
	float3 ForegroundColor = lerp(float3(1, 0, 0), float3(1, 1, 0), PseudoRandom(PixelPos)); 

	int2 tile = (int2)floor(PixelPos / 12.0f);
	int2 experiment = (int2)floor(tile / 5.0f);

	if(experiment.x >= 0 && experiment.y >= 0 && experiment.x < 10 && experiment.y < 5)
	{
		int2 localtile = uint2(tile) % 5;

		bool bForeground = localtile.x == 2 && localtile.y == 2;

		Color.rgb = bForeground ? ForegroundColor : BackgroundColor;

		// depth
		Color.a = bForeground ? 100.0f : 1000.0f;

		bool bLeftSide = experiment.x < 5;

		if(!bLeftSide)
		{
			experiment.x -= 5;
		}

		float ForegroundAngle = (experiment.x - 1) * (6.283f / 12);
		float BackgroundAngle = (experiment.y - 1) * (6.283f / 12) + 3.1415f/2;

		// ForegroundR with very small amounts needs extra testing so we do a non linear scale
		float ForegroundR = pow(experiment.x / 5.0f, 2);
		float BackgroundR = pow(experiment.y / 5.0f, 2);

		float2 ForegroundXY = ForegroundR * float2(sin(ForegroundAngle), cos(ForegroundAngle));
		float2 BackgroundXY = BackgroundR * float2(sin(BackgroundAngle), cos(BackgroundAngle));

		BackgroundVelocity.xy = BackgroundXY;

		if(bLeftSide)
		{
			ObjectVelocity.xy = ForegroundXY;
			AvgObject = bForeground;
		}
		else
		{
			ObjectVelocity.xy = bForeground ? ForegroundXY : BackgroundXY;
			AvgObject = 1.0f;
		}
	}

#endif
}

// Scale must be a power of two
float3 PeriodicWorldPosition(float3 TranslatedWorldPosition, float Scale)
{
	float3 Offset = -DFFmodByPow2Demote(PrimaryView.PreViewTranslation, Scale);
	return (TranslatedWorldPosition + Offset) / Scale;
}

// used to visualize the motion blur
// @return 0/1
float Compute3DCheckerBoard(float3 Pos)
{
	float3 TiledWorldPos = frac(Pos) > 0.5f;
	return (float)((uint)dot(float3(1,1,1), TiledWorldPos) % 2);
}

//------------------------------------------------------- ENTRY POINT

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const float TileSize = TILE_SIZE;

	float2 DistortedScreenPos = ApplyScreenTransform(SvPosition.xy, SvPositionToScreenPos);
	float2 UndistortedScreenPos = ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedScreenPos);

	float2 VelocityUV = ApplyScreenTransform(UndistortedScreenPos, ScreenPosToVelocityUV);

	float2 PixelPos = DistortedScreenPos * Velocity_ScreenPosToViewportScale + Velocity_ScreenPosToViewportBias;
	float2 PixelPosAtTileCenter = PixelPos - (frac(PixelPos / TileSize) - 0.5f) * TileSize;

	float2 DistortedScreenPosAtTileCenter = (PixelPosAtTileCenter - Velocity_ScreenPosToViewportBias) / Velocity_ScreenPosToViewportScale;
	float2 UndistortedScreenPosAtTileCenter = ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedScreenPosAtTileCenter);

	float2 VelocityUVAtTileCenter = ApplyScreenTransform(UndistortedScreenPosAtTileCenter, ScreenPosToVelocityUV);

	// World aligned checkerboards
	OutColor = float4(0, 0, 0, 1);
	if (CheckerboardEnabled)
	{
		float DeviceZ = DepthTexture.SampleLevel(SharedDepthSampler, VelocityUV, 0).r;
		float SceneDepth = ConvertFromDeviceZ(DeviceZ);

		float3 ScreenVector = mul(float3(UndistortedScreenPos, 1), DFToFloat3x3(PrimaryView.ScreenToWorld)).xyz;

		// world space position of the current pixel
		float3 OffsetWorldPos = ScreenVector * SceneDepth;

		float3 TranslatedWorldPos = PrimaryView.TranslatedWorldCameraOrigin + OffsetWorldPos;
		float3 TiledPos = frac(PeriodicWorldPosition(TranslatedWorldPos, pow(2,12))); // convert TWP into worldspace repeating tiles of size 2^12
		float3 WorldCheckerboard = Compute3DCheckerBoard(TiledPos) * 0.1f + Compute3DCheckerBoard(TiledPos * pow(2,3)) * 0.3f + Compute3DCheckerBoard(TiledPos * pow(2,6)) * 0.6f;
		OutColor = float4(lerp(WorldCheckerboard, float3(0,0,0), 0.7f), 1);
	}

	bool bSelectorOpaqueAtTileCenter = VelocityTexture.SampleLevel(SharedVelocitySampler, VelocityUVAtTileCenter, 0).x > 0;
	bool bSelectorOpaque = VelocityTexture.SampleLevel(SharedVelocitySampler, VelocityUV, 0).x > 0;

	// relative, in screen space -1...1 -1..1, can be even outside of that range, points into the movement direction
	float2 VelocityAtTileCenter;
	{
		VelocityAtTileCenter = DecodeVelocityFromTexture(VelocityTexture.SampleLevel(SharedVelocitySampler, VelocityUVAtTileCenter, 0)).xy;

		// reconstruct from camera motion if we don't have velocity data
		if(!bSelectorOpaqueAtTileCenter)
		{
			float DeviceZ = DepthTexture.SampleLevel(SharedDepthSampler, VelocityUVAtTileCenter, 0).r;

			// UV is per tile
			float4 ThisClip = float4(UndistortedScreenPosAtTileCenter, DeviceZ, 1);
			float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
			float2 PrevScreen = PrevClip.xy / PrevClip.w;

			// points into the movement direction
			VelocityAtTileCenter = UndistortedScreenPosAtTileCenter - PrevScreen;
		}
	}

	// tint yellow if velocity data is stored in texture
	{
		float3 TintColor = bSelectorOpaque ? float3(0.5f, 0.5f, 0.2f) : float3(0.5f, 0.5f, 0.5f);
	
		OutColor.rgb = lerp(OutColor.rgb, TintColor, 0.45f);
	}

#if VISUALIZE_COLOR_ENCODING
	{
		float2 Velocity = DecodeVelocityFromTexture(VelocityTexture.SampleLevel(SharedVelocitySampler, VelocityUV, 0)).xy;
		{
			// reconstruct from camera motion if we don't have velocity data
			if (!bSelectorOpaque)
			{
				float DeviceZ = DepthTexture.SampleLevel(SharedDepthSampler, VelocityUV, 0).r;

				// UV is per tile
				float4 ThisClip = float4(UndistortedScreenPos, DeviceZ, 1);
				float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
				float2 PrevScreen = PrevClip.xy / PrevClip.w;

				// points into the movement direction
				Velocity = UndistortedScreenPos - PrevScreen;
			}
		}

		Velocity = Velocity * View.ViewSizeAndInvSize.xy;

		float Gamma = 1/2.2f;
		float Magnitude = clamp(length(Velocity)/COLOR_METER_LENGTH, 0.0f, 1.0f);
		float AdjustedMagnitude = pow(Magnitude, Gamma);
		float Satuation = 1.0f;
		int2 TexCoord = SvPosition.xy;

		if (VisualizeMode == 1)
		{
			float h = atan2(Velocity.y, Velocity.x) / 3.1415927f;
			h = h < 0 ? ((h + 2) / 2) : h / 2.0;
			float s = Satuation;
			float v = AdjustedMagnitude;

			float3 ColorEncoding = HSV_2_LinearRGB_Smooth(float3(h, s, v));

			// Use per pixel encoding if the velocity is available, otherwise fallback to the yellow checkerbaord.
			OutColor.rgb = lerp(OutColor.rgb, ColorEncoding, length(Velocity) > 0.0f);

			if (length(TexCoord.xy - (COLOR_METER_LENGTH + COLOR_METER_OFFSET)) < COLOR_METER_LENGTH)
			{
				float2 Vector = TexCoord.xy - COLOR_METER_LENGTH;

				h = atan2(-Vector.y, Vector.x) / 3.1415927f;
				h = h < 0 ? ((h + 2) / 2) : h / 2.0;
				s = Satuation;
				v = pow(length(float3(Vector, 0)) / COLOR_METER_LENGTH, Gamma);
				OutColor = float4(HSV_2_LinearRGB_Smooth(float3(h, s, v)), 1.0f);
			}
		}
		else if (VisualizeMode == 2)
		{
			float3 ColorEncoding = ColorMapPlasma(AdjustedMagnitude);

			// Use per pixel encoding if the velocity is available, otherwise fallback to the yellow checkerbaord.
			OutColor.rgb = lerp(OutColor.rgb, ColorEncoding, length(Velocity) > 0.0f);

			uint2 ColorMeterPosition = TexCoord.xy - (COLOR_METER_OFFSET);
			if (all(ColorMeterPosition < uint2(COLOR_METER_LENGTH, 20)) &&
				all(ColorMeterPosition >= uint2(0, 0)))
			{
				Magnitude = (TexCoord.x - COLOR_METER_OFFSET);
				AdjustedMagnitude = pow( Magnitude / COLOR_METER_LENGTH, Gamma);
				OutColor = float4(ColorMapPlasma(AdjustedMagnitude), 1.0f);
			}
		}
	}
#endif 

	// tile center
	{
		float2 Delta = PixelPos - PixelPosAtTileCenter;

		float HMask = lerp(saturate(abs(Delta.y)), 1, saturate(abs(Delta.x) / TileSize * 4));
		float VMask = lerp(saturate(abs(Delta.x)), 1, saturate(abs(Delta.y) / TileSize * 4));

		float Dist = length(Delta);
		OutColor.rgb *= lerp(1, 0.3f, saturate(3 - Dist));
	}

	float3 LineColor = bSelectorOpaqueAtTileCenter ? float3(1,1,0) : float3(0.7,0.7,0.7);

	// points into the movement direction
	float2 PixelDirection = VelocityAtTileCenter * Velocity_ScreenPosToViewportScale;

	// arrow
	{
		float2 PerpPixelDirection = float2(PixelDirection.y, -PixelDirection.x);
		float2 DirectionInTile = PixelPos - PixelPosAtTileCenter;

		float DistOnLine = dot(normalize(-PixelDirection), DirectionInTile) + length(PixelDirection);

		bool bArrowHead = DistOnLine < 8;

		float LocalThickness = 1 + (frac(DistOnLine/8)*8)*0.25f;

		float PerpDirectionMask = saturate(LocalThickness - abs(dot(normalize(PerpPixelDirection), DirectionInTile)));
		float DirectionMask = saturate(length(PixelDirection) - length(DirectionInTile));

		float3 LineMask = PerpDirectionMask * DirectionMask;
		OutColor.rgb = lerp(OutColor.rgb, LineColor, LineMask);
	}

	// previous pos is a dot
	{
		float3 DotColor = float3(0,1,0);
		// PixelPos of the previous position
		float2 PreviousPixelPos = PixelPosAtTileCenter - PixelDirection;
		float Dist = length(PreviousPixelPos - PixelPos);
		OutColor.rgb = lerp(OutColor.rgb, LineColor, saturate(4 - Dist));
		OutColor.rgb = lerp(OutColor.rgb, 0, saturate(2.5f - Dist));
	}
}
