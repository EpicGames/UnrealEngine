// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ScreenPass.ush"
#include "../LensDistortion.ush"


//------------------------------------------------------- PARAMETERS

Texture2D<float2> PrevDistortingDisplacementTexture;
SamplerState      PrevDistortingDisplacementSampler;
Texture2D<float2> UndistortingDisplacementTexture;
SamplerState      UndistortingDisplacementSampler;

Texture2D ColorTexture;
Texture2D VelocityTexture;
Texture2D DepthTexture;
Texture2D PrevColorTexture;

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Velocity)
SCREEN_PASS_TEXTURE_VIEWPORT(PrevColor)

SamplerState ColorSampler;
SamplerState VelocitySampler;
SamplerState DepthSampler;
SamplerState PrevColorSampler;

FScreenTransform SvPositionToScreenPos;
FScreenTransform ScreenPosToVelocity;
FScreenTransform ScreenPosToColor;
FScreenTransform PrevScreenPosToPrevColor;
uint VisualizeType;


//------------------------------------------------------- FUNCTIONS

float2 ComputeStaticVelocity(float2 ScreenPos, float DeviceZ)
{
	float3 PosN = float3(ScreenPos, DeviceZ);

	float4 ThisClip = float4(PosN, 1);
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float2 PrevScreen = PrevClip.xy / PrevClip.w;
	return AdjustClipToPrevClipForProjectionType(PosN.xy - PrevScreen, DeviceZ);
}


//------------------------------------------------------- ENTRY POINT

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 DistortedScreenPos = ApplyScreenTransform(SvPosition.xy, SvPositionToScreenPos);
	float2 UndistortedScreenPos = ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedScreenPos);

	BRANCH
	if (VisualizeType == 0) // EVisualizeMotionVectors::ReprojectionAlignment
	{
		float2 ColorTextureUV = ApplyScreenTransform(DistortedScreenPos, ScreenPosToColor);
		float2 VelocityTextureUV = ApplyScreenTransform(UndistortedScreenPos, ScreenPosToVelocity);

		ColorTextureUV = clamp(ColorTextureUV, Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);
		VelocityTextureUV = clamp(VelocityTextureUV, Velocity_UVViewportBilinearMin, Velocity_UVViewportBilinearMax);

		float4 Color = ColorTexture.SampleLevel(ColorSampler, ColorTextureUV, 0.0);
		float4 EncodedVelocity = VelocityTexture.SampleLevel(VelocitySampler, VelocityTextureUV, 0.0);
		float DeviceZ = DepthTexture.SampleLevel(DepthSampler, VelocityTextureUV, 0.0).r;
	
		float2 ScreenVelocity = ComputeStaticVelocity(UndistortedScreenPos, DeviceZ);
		bool bIsRenderedVelocity = EncodedVelocity.x > 0.0;
		if (bIsRenderedVelocity)
		{
			ScreenVelocity = DecodeVelocityFromTexture(EncodedVelocity).xy;
		}

		float2 UndistortedPrevScreenPos = UndistortedScreenPos - ScreenVelocity;
		float2 DistortedPrevScreenPos = ApplyLensDistortionOnScreenPos(PrevDistortingDisplacementTexture, PrevDistortingDisplacementSampler, UndistortedPrevScreenPos);
		float2 PrevColorTextureUV = ApplyScreenTransform(UndistortedPrevScreenPos, PrevScreenPosToPrevColor);
		PrevColorTextureUV = clamp(PrevColorTextureUV, Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);

		float4 PrevColor = PrevColorTexture.SampleLevel(PrevColorSampler, PrevColorTextureUV, 0.0);

		float Luma = Luminance(Color.rgb);
		float PrevLuma = Luminance(PrevColor.rgb);

		uint2 SubTileCoord = (uint2(SvPosition.xy) >> 4) & 0x1;
		float Checker = SubTileCoord.x == SubTileCoord.y ? 1.0 : 0.0;

		if (bIsRenderedVelocity)
		{
			OutColor.rgb = (Luma + PrevLuma).xxx * 0.5 + abs(Color.xyz - PrevColor.xyz);
			//OutColor.rgb = (Color + PrevColor) * 0.5;
			OutColor.a = min(Color.a, PrevColor.a);
		}
		else
		{
			OutColor = float4(Luma.xxx * (0.25 + 0.25 * Checker), Color.a);
		}
	}
	#if VELOCITY_ENCODE_HAS_PIXEL_ANIMATION
	else if (VisualizeType == 1) // EVisualizeMotionVectors::HasPixelAnimationFlag
	{
		// Don't apply lens distortion on HasPixelAnimation to be consistent with the rest of the VisualizeTemporalUpscaler's inputs.
		float2 VelocityTextureUV = ApplyScreenTransform(DistortedScreenPos, ScreenPosToVelocity);
		VelocityTextureUV = clamp(VelocityTextureUV, Velocity_UVViewportBilinearMin, Velocity_UVViewportBilinearMax);
		
		float4 EncodedVelocity = VelocityTexture.SampleLevel(VelocitySampler, VelocityTextureUV, 0.0);
		bool bHasPixelAnimation = DecodeHasPixelAnimationFromVelocityTexture(EncodedVelocity);
		#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
			bool bDisableAnimationAA = DecodeDisableAnimationAAFromVelocityTexture(EncodedVelocity);
		#else
			bool bDisableAnimationAA = false;
		#endif

		OutColor = bHasPixelAnimation ? 1.0 : 0.0;
		OutColor.rgb = lerp(OutColor.rgb,float3(1,0,0), bDisableAnimationAA ? 0.5 : 0); 
	}
	#endif
	else
	{
		OutColor = 0.0;
	}
}
