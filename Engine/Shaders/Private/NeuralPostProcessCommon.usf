// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NeuralPostProcessCommon.usf: Shaders for neural post process
=============================================================================*/

#ifndef MATERIAL_NEURAL_POST_PROCESS
#define MATERIAL_NEURAL_POST_PROCESS 0
#endif

#ifndef NEURAL_POSTPROCESS_PREPASS
#define NEURAL_POSTPROCESS_PREPASS 0
#endif

#if MATERIAL_NEURAL_POST_PROCESS

// Input buffer/textue set by the prepass
RWTexture2D<float4> RWNeuralTexture;
RWBuffer<float> InputNeuralBuffer;
RWBuffer<uint> NeuralSourceType;
float4 InputNeuralBufferDimension;

// Buffer/textures read by the main pass
Texture2D<float4> NeuralTexture;
Buffer<float> OutputNeuralBuffer;
float4 OutputNeuralBufferDimension;

#define SOURCE_TYPE_TEXTURE 0
#define SOURCE_TYPE_BUFFER  1

#endif

void UpdateNeuralProfileSourceType(int InputSourceType)
{
#if MATERIAL_NEURAL_POST_PROCESS && NEURAL_POSTPROCESS_PREPASS
	NeuralSourceType[0] = InputSourceType;
#endif
}


// Save a 3 channel data to OutputNeuralBuffer from StartChannel to StartChannel+2 
void SaveToNeuralBuffer(int Batch, int StartChannel, float2 BufferWH, float3 Value)
{
#if MATERIAL_NEURAL_POST_PROCESS && NEURAL_POSTPROCESS_PREPASS
	float2 HWIndex = BufferWH.yx * InputNeuralBufferDimension.zw - 0.5f;

	// point sampling
	const int	IX = (int)clamp(HWIndex.y, 0, InputNeuralBufferDimension.w - 1);
	const int	IY = (int)clamp(HWIndex.x, 0, InputNeuralBufferDimension.z - 1);

	int BufferHeight = InputNeuralBufferDimension.z;
	int BufferWidth = InputNeuralBufferDimension.w;
	int BufferGridSize = BufferHeight * BufferWidth;
	int NumChannels = InputNeuralBufferDimension.y;
	int BatchSize = NumChannels * BufferGridSize;

	int Idx = (Batch * BatchSize + StartChannel * BufferGridSize + IY * BufferWidth + IX);
	int Offset = BufferGridSize;

	// TODO: array of struct vs struct of array performance evaluation.
	InputNeuralBuffer[Idx] = Value[0];
	const int MaxOffset = min(3, InputNeuralBufferDimension.y - StartChannel) - 1;
	for (int i = 1; i <= MaxOffset; ++i)
	{
		InputNeuralBuffer[Idx + i * Offset] = Value[i];
	}
#endif

}

void ReadFromNeuralBuffer(int Batch, int StartChannel, float2 BufferWH, inout float3 Value)
{
#if MATERIAL_NEURAL_POST_PROCESS && MATERIAL_NEURAL_POST_PROCESS
	float2 HWIndex = BufferWH.yx * OutputNeuralBufferDimension.zw - 0.5f;
	
	// point sampling
	const int	IX = (int)clamp(HWIndex.y, 0, OutputNeuralBufferDimension.w - 1);
	const int	IY = (int)clamp(HWIndex.x, 0, OutputNeuralBufferDimension.z - 1);

	int BufferHeight = OutputNeuralBufferDimension.z;
	int BufferWidth = OutputNeuralBufferDimension.w;
	int BufferGridSize = BufferHeight * BufferWidth;
	int NumChannels = OutputNeuralBufferDimension.y;
	int BatchSize = NumChannels * BufferGridSize;

	int Idx = (Batch * BatchSize + StartChannel * BufferGridSize + IY * BufferWidth + IX);
	int Offset = BufferGridSize;

	// TODO: array of struct vs struct of array performance evaluation.
	const int MaxOffset = min(3, OutputNeuralBufferDimension.y- StartChannel) - 1;
	for (int i = 0; i <= MaxOffset; ++i)
	{
		Value[i] = OutputNeuralBuffer[Idx + i * Offset];
	}
#else
	Value = 0;
#endif
}