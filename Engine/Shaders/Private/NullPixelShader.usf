// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NullPixelShader.usf: Dummy pixel shader to use instead of SetPixelShader(NULL).
=============================================================================*/

#include "Common.ush"

float4 Main() : SV_Target0
{
	return 0;
}
