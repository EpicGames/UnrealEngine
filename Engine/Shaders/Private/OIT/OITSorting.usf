// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// RECOMPILE HASH
#pragma message("UESHADERMETADATA_VERSION 78C3EBBB-59EA-459D-A2D4-67170905D7FC")

#include "../Common.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////

#define PRIMITIVE_TRILIST  0
#define PRIMITIVE_TRISTRIP 1
#define ENCODING_TYPE 1

#ifndef SORTING_SLICE_COUNT
	#error SORTING_SLICE_COUNT needs to be defined 
#endif

#define BACK_TO_FRONT 0
#define FRONT_TO_BACK 1

#if ENCODING_TYPE == 1
	#define FPackedSliceAndOffset uint
#else
	#define FPackedSliceAndOffset uint2
#endif

#if PERMUTATION_DEBUG || SHADER_DEBUG
#include "../ShaderPrint.ush"
#include "../ColorMap.ush"
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FTriIndex
{
	uint I0;
	uint I1;
	uint I2;
};

FTriIndex GetTriangleIndices(uint InPrimitiveId, uint InFirstIndex, uint InPrimitiveType)
{
	FTriIndex Out = (FTriIndex)0;
	if (InPrimitiveType == PRIMITIVE_TRILIST)
	{
		Out.I0 = InFirstIndex + InPrimitiveId * 3;
		Out.I1 = Out.I0 + 1;
		Out.I2 = Out.I0 + 2;
	}
	else if (InPrimitiveType == PRIMITIVE_TRISTRIP)
	{
		Out.I0 = (InFirstIndex < 3 || InPrimitiveId < 1) ? InPrimitiveId : InPrimitiveId + 2;
		Out.I1 = Out.I0 + 1;
		Out.I2 = Out.I0 + 2;
	}

	return Out;
}

bool IsValid(FTriIndex InTri, uint InNumIndices)
{
	return InTri.I0 < InNumIndices && InTri.I1 < InNumIndices && InTri.I2 < InNumIndices;
}

#if SORTING_SLICE_COUNT > 256
#error Slice index is encoded onto 8bits and no longer fit into stored data
#endif

FPackedSliceAndOffset PackSliceAndOffset(uint InSliceIndex, uint InLocalOffset)
{
#if ENCODING_TYPE
	return (InLocalOffset & 0xFFFFFF) | ((InSliceIndex&0xFF) << 24);
#else
	return uint2(InSliceIndex, InLocalOffset);
#endif
}

void UnpackSliceAndOffset(FPackedSliceAndOffset In, inout uint OutSliceIndex, inout uint OutLocalOffset)
{
#if ENCODING_TYPE
	OutSliceIndex  = (In>>24) & 0xFF;
	OutLocalOffset = In & 0xFFFFFF;
#else
	OutSliceIndex  = In.x;
	OutLocalOffset = In.y;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Debug

struct FOITDebugData
{
	uint Index;
	uint PrimitiveCount;
	uint Type;
	uint SizeInBytes;

	uint FirstIndex;
	uint BaseVertexIndex;
	uint MinVertexIndex;
	uint MaxVertexIndex;
};

void AddDebugInfo(RWBuffer<uint> InDebugData, FOITDebugData In)
{
	uint Index = 0;
	InterlockedAdd(InDebugData[0], 1, Index);

	const uint Index3 = 1 + Index * 7u;
	InDebugData[Index3 + 0] = In.PrimitiveCount;
	InDebugData[Index3 + 1] = In.Type;
	InDebugData[Index3 + 2] = In.SizeInBytes;

	InDebugData[Index3 + 3] = In.FirstIndex;
	InDebugData[Index3 + 4] = In.BaseVertexIndex;
	InDebugData[Index3 + 5] = In.MinVertexIndex;
	InDebugData[Index3 + 6] = In.MaxVertexIndex;
}

uint GetDebugInfoCount(Buffer<uint> InDebugData)
{
	return InDebugData[0];
}

FOITDebugData GetDebugInfo(Buffer<uint> InDebugData, uint Index)
{
	FOITDebugData Out = (FOITDebugData)0;
	if (Index < InDebugData[0])
	{
		const uint Index3 = 1 + Index * 7u;
		Out.Index			= Index;
		Out.PrimitiveCount	= InDebugData[Index3 + 0];
		Out.Type			= InDebugData[Index3 + 1];
		Out.SizeInBytes	= InDebugData[Index3 + 2];

		Out.FirstIndex		= InDebugData[Index3 + 3];
		Out.BaseVertexIndex	= InDebugData[Index3 + 4];
		Out.MinVertexIndex	= InDebugData[Index3 + 5];
		Out.MaxVertexIndex	= InDebugData[Index3 + 6];
	}

	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_SCAN

#if PERMUTATION_DEBUG
float4x4 ViewToWorld;
float3 WorldBound_Min;
float3 WorldBound_Max;
float3 ViewBound_Min;
float3 ViewBound_Max;
#endif

float4x4 LocalToWorld; // GPU scene buffer instead? How?
float4x4 LocalToView;

uint SourcePrimitiveType;
uint NumPrimitives;
uint NumIndices;
uint SourceFirstIndex;
uint SourceBaseVertexIndex;
uint SourceMinVertexIndex;
uint SourceMaxVertexIndex;
uint SortType;
uint SortedIndexBufferSizeInByte;

float ViewBoundMaxZ;
float ViewBoundMinZ;

Buffer<float> PositionBuffer;
Buffer<uint> IndexBuffer;

RWBuffer<uint> OutIndexBuffer;
RWBuffer<uint> OutSliceCounterBuffer;
RWBuffer<FPackedSliceAndOffset> OutPrimitiveSliceBuffer;

RWBuffer<uint> OutDebugData;

groupshared uint s_SliceIndices[SORTING_SLICE_COUNT];
groupshared uint s_SliceOffsets[SORTING_SLICE_COUNT];

[numthreads(SORTING_SLICE_COUNT, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint GroupThreadId : SV_GroupIndex)
{
	uint PrimitiveId = DispatchThreadId.x;
	uint LocalOffset = 0;
	uint SliceIndex = 0;

	// 0. Clear local allocation (shared-memory)
	if (GroupThreadId < SORTING_SLICE_COUNT)
	{
		s_SliceIndices[GroupThreadId] = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	// 1. Compute slice index, and local allocation (shared-memory)
	const bool bIsValid = PrimitiveId < NumPrimitives;
	if (bIsValid)
	{
		const FTriIndex Tri = GetTriangleIndices(PrimitiveId, SourceFirstIndex, SourcePrimitiveType);
		const uint I0 = 3 * clamp(IndexBuffer[Tri.I0] + SourceBaseVertexIndex, SourceMinVertexIndex, SourceMaxVertexIndex);
		const uint I1 = 3 * clamp(IndexBuffer[Tri.I1] + SourceBaseVertexIndex, SourceMinVertexIndex, SourceMaxVertexIndex);
		const uint I2 = 3 * clamp(IndexBuffer[Tri.I2] + SourceBaseVertexIndex, SourceMinVertexIndex, SourceMaxVertexIndex);

		// Position buffer store data as a float, not float3
		const float3 P0 = float3(PositionBuffer[I0], PositionBuffer[I0+1], PositionBuffer[I0+2]);
		const float3 P1 = float3(PositionBuffer[I1], PositionBuffer[I1+1], PositionBuffer[I1+2]);
		const float3 P2 = float3(PositionBuffer[I2], PositionBuffer[I2+1], PositionBuffer[I2+2]);

		// Compute triangle center in view space
		const float3 LocalP = (P0 + P1 + P2) / 3;
		const float3 ViewP = mul(float4(LocalP, 1), LocalToView).xyz;

		const float fSliceIndex = saturate((ViewP.z - ViewBoundMinZ) / (ViewBoundMaxZ - ViewBoundMinZ));
		SliceIndex = SORTING_SLICE_COUNT * (SortType == BACK_TO_FRONT ? (1.f - fSliceIndex) : fSliceIndex);
		SliceIndex = clamp(SliceIndex, 0, SORTING_SLICE_COUNT - 1);

		#if PERMUTATION_DEBUG
		{
			FShaderPrintContext Ctx = InitShaderPrintContext();
			const float4 TriColor = float4(ColorMapViridis(SliceIndex / float(SORTING_SLICE_COUNT - 1)), 1);

			const float3 WP0 = mul(float4(P0, 1), LocalToWorld).xyz;
			const float3 WP1 = mul(float4(P1, 1), LocalToWorld).xyz;
			const float3 WP2 = mul(float4(P2, 1), LocalToWorld).xyz;
			AddLineTriangleWS(Ctx, WP0, WP1, WP2, TriColor);
		}
		#endif

		InterlockedAdd(s_SliceIndices[SliceIndex], 1, LocalOffset);
	}

	#if PERMUTATION_DEBUG
	if (all(DispatchThreadId == 0))
	{
		AddAABBWS(WorldBound_Min, WorldBound_Max, ColorYellow);
		AddOBBWS(ViewBound_Min, ViewBound_Max, ColorCyan, ViewToWorld);

		FOITDebugData DebugData;
		DebugData.PrimitiveCount	= NumPrimitives;
		DebugData.Type				= SourcePrimitiveType;
		DebugData.SizeInBytes		= SortedIndexBufferSizeInByte;

		DebugData.FirstIndex		= SourceFirstIndex;
		DebugData.BaseVertexIndex	= SourceBaseVertexIndex;
		DebugData.MinVertexIndex	= SourceMinVertexIndex;
		DebugData.MaxVertexIndex	= SourceMaxVertexIndex;

		AddDebugInfo(OutDebugData, DebugData);
	}
	#endif

	GroupMemoryBarrierWithGroupSync();

	// 2. Global allocation (global-memory)
	if (GroupThreadId < SORTING_SLICE_COUNT)
	{
		if (s_SliceIndices[GroupThreadId] > 0)
		{
			InterlockedAdd(OutSliceCounterBuffer[GroupThreadId], s_SliceIndices[GroupThreadId], s_SliceOffsets[GroupThreadId]);
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// 3. Write slice index & relative offset per primitive
	if (bIsValid)
	{
		// Write slice index and the primitive offset so that we don't need to atomic ops during the final write-out pass
		const uint GlobalRelativeOffset = s_SliceOffsets[SliceIndex] + LocalOffset;
		OutPrimitiveSliceBuffer[PrimitiveId] = PackSliceAndOffset(SliceIndex, GlobalRelativeOffset);
	}
}
#endif // SHADER_SCAN

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_ALLOCATE
Buffer<uint> SliceCounterBuffer;
RWBuffer<uint> SliceOffsetsBuffer;

groupshared uint s_Counters[SORTING_SLICE_COUNT];
groupshared uint s_Offsets[SORTING_SLICE_COUNT];

[numthreads(SORTING_SLICE_COUNT, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint SliceIndex = DispatchThreadId.x;
	const bool bIsValid = SliceIndex < SORTING_SLICE_COUNT;

	if (bIsValid)
	{
		s_Counters[SliceIndex] = SliceCounterBuffer[SliceIndex];
	}

	GroupMemoryBarrierWithGroupSync();

	// TODO: parallel prefix sum
	if (SliceIndex == 0)
	{
		uint AccumPrimitiveCount = 0;
		for (uint SliceIt = 0; SliceIt < SORTING_SLICE_COUNT; ++SliceIt)
		{
			s_Offsets[SliceIt] = AccumPrimitiveCount;
			AccumPrimitiveCount += s_Counters[SliceIt];
		}
	}

	GroupMemoryBarrierWithGroupSync();

	if (bIsValid)
	{
		SliceOffsetsBuffer[SliceIndex] = s_Offsets[SliceIndex];
	}
}
#endif // SHADER_ALLOCATE

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_WRITE

uint SourcePrimitiveType;
uint NumPrimitives;
uint NumIndices;
uint SrcFirstIndex;
uint DstFirstIndex;
uint SortType;

Buffer<uint>  SliceOffsetsBuffer;
Buffer<FPackedSliceAndOffset> PrimitiveSliceBuffer;

Buffer<uint> IndexBuffer;
RWBuffer<uint> OutIndexBuffer;

[numthreads(256, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint SrcPrimitiveId = DispatchThreadId.x;
	if (SrcPrimitiveId >= NumPrimitives)
	{
		return;
	}

	uint SliceIndex = 0;
	uint PrimitiveOffset = 0;
	UnpackSliceAndOffset(PrimitiveSliceBuffer[SrcPrimitiveId], SliceIndex, PrimitiveOffset);

	uint SlicePrimitiveOffset = 0;
	if (SliceIndex < SORTING_SLICE_COUNT)
	{
		SlicePrimitiveOffset = SliceOffsetsBuffer[SliceIndex];
	}

	const uint DstPrimitiveId = SlicePrimitiveOffset + PrimitiveOffset;

	const FTriIndex SrcTri = GetTriangleIndices(SrcPrimitiveId, SrcFirstIndex, SourcePrimitiveType);
	const FTriIndex DstTri = GetTriangleIndices(DstPrimitiveId, DstFirstIndex, PRIMITIVE_TRILIST); // Always write out triangle list, since sorting break strip topology

	if (IsValid(DstTri, NumIndices))
	{
		OutIndexBuffer[DstTri.I0] = IndexBuffer[SrcTri.I0];
		OutIndexBuffer[DstTri.I1] = IndexBuffer[SrcTri.I1];
		OutIndexBuffer[DstTri.I2] = IndexBuffer[SrcTri.I2];
	}
}
#endif // SHADER_WRITE


///////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_DEBUG

uint VisibleInstances;
uint VisiblePrimitives;
uint VisibleIndexSizeInBytes;

uint AllocatedBuffers;
uint AllocatedIndexSizeInBytes;

uint UnusedBuffers;
uint UnusedIndexSizeInBytes;

uint TotalEntries;

Buffer<uint> DebugData;

void PrintSize(inout FShaderPrintContext Context, uint SizeInBytes, FFontColor Color)
{
	if (SizeInBytes < 1000)
	{
		Print(Context, SizeInBytes, Color, 5, 2);
		Print(Context, TEXT("   bytes "), Color);
	}
	else if (SizeInBytes < 1000000)
	{
		Print(Context, SizeInBytes / 1000.f, Color, 5, 2);
		Print(Context, TEXT("  Kbytes "), Color);
	}
	else
	{
		Print(Context, SizeInBytes / 1000000.f, Color, 5, 2);
		Print(Context, TEXT("  Mbytes "), Color);
	}
}

[numthreads(1, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

	// Total NumPrimitive | Total Size 0.4 Mb

	// Visible
	{
		Print(Context, TEXT("Visible"), FontOrange); Newline(Context);
		Print(Context, TEXT("#Sort slices   : "), FontWhite); Print(Context, uint(SORTING_SLICE_COUNT), FontSilver); Newline(Context);
		Print(Context, TEXT("Instances      : "), FontWhite); Print(Context, VisibleInstances, FontSilver); Newline(Context);
		Print(Context, TEXT("Primitives     : "), FontWhite); Print(Context, VisiblePrimitives, FontSilver); Newline(Context);
		Print(Context, TEXT("Size           : "), FontYellow); PrintSize(Context, VisibleIndexSizeInBytes, FontYellow); Newline(Context);
	}
	Newline(Context);

	// Memory
	{
		Print(Context, TEXT("Memory"), FontOrange); Newline(Context);
		Print(Context, TEXT("Alloc. Buffers : "), FontWhite); Print(Context, AllocatedBuffers, FontSilver, 3, 0); Print(Context, TEXT("/"), FontSilver); Print(Context, TotalEntries, FontSilver, 3, 0); Newline(Context);
		Print(Context, TEXT("Alloc. Size    : "), FontWhite); PrintSize(Context, AllocatedIndexSizeInBytes, FontSilver); Newline(Context);
		Print(Context, TEXT("Unused Buffers : "), FontWhite); Print(Context, UnusedBuffers, FontSilver); Newline(Context);
		Print(Context, TEXT("Unused Size    : "), FontWhite); PrintSize(Context, UnusedIndexSizeInBytes, FontSilver); Newline(Context);
		Print(Context, TEXT("Total Size     : "), FontYellow); PrintSize(Context, AllocatedIndexSizeInBytes + UnusedIndexSizeInBytes, FontYellow); Newline(Context);
	}
	Newline(Context);

	// 1: NumPrimitive | PrimitiveType | 0.4 Mb
	{
		Print(Context, TEXT("Visible (details)"), FontSilver);
		Newline(Context);

		Print(Context, TEXT("Index       "));
		Print(Context, TEXT("Primitive   ")); 
		Print(Context, TEXT("Type  "));
		Print(Context, TEXT("Size          "));

		Print(Context, TEXT("FirstIdx    "));
		Print(Context, TEXT("BaseIdx     "));
		Print(Context, TEXT("MinVtxIdx   "));
		Print(Context, TEXT("MaxVtxIdx   "));
		Print(Context, TEXT("#Vertex     "));
		Newline(Context);

		for (uint Index = 0; Index < VisibleInstances; ++Index)
		{
			const FOITDebugData Data = GetDebugInfo(DebugData, Index);

			Print(Context, Data.Index, FontWhite);
			Print(Context, Data.PrimitiveCount, FontSilver);
			if (Data.Type == 0)
				Print(Context, TEXT("List  "), FontSilver);
			else
				Print(Context, TEXT("Strip "), FontSilver);

			PrintSize(Context, Data.SizeInBytes, FontYellow);

			Print(Context, Data.FirstIndex, FontSilver);
			Print(Context, Data.BaseVertexIndex, FontSilver);
			Print(Context, Data.MinVertexIndex, FontSilver);
			Print(Context, Data.MaxVertexIndex, FontSilver);
			Print(Context, Data.MaxVertexIndex - Data.MinVertexIndex + 1, FontSilver);

			Newline(Context);
		}
	}
}
#endif // SHADER_DEBUG