// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

#define OIT_ENABLED 1
#define MAX_SAMPLE_COUNT 8
#define OIT_PASS_REGULAR 1
#define OIT_PASS_SEPARATE 2

#include "OITCommon.ush"
#include "ColorMap.ush"

int2 Resolution;
uint MaxSideSampleCount;
uint MaxSampleCount;
uint Method;
uint PassType;
uint SupportedPass;

Texture2DArray<uint> SampleDataTexture;
Texture2D<uint>	SampleCountTexture;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Combine

#if SHADER_OIT_COMBINE


#if PERMUTATION_MODULATE
RWTexture2D<float4> OutLightingTexture;
RWTexture2D<float4> OutTransmittanceTexture;
#else
RWTexture2D<float4> OutColorTexture;
#endif

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{	
	const uint2 PixelCoord = DispatchThreadId.xy;
	if (any(PixelCoord.xy >= uint2(Resolution)))
		return;

	FOITSample Samples[MAX_SAMPLE_COUNT];

	const uint  SampleCount		= min(MaxSampleCount, SampleCountTexture.Load(uint3(PixelCoord, 0)));
	const uint2 SampleBaseCoord	= PixelCoord * MaxSideSampleCount;

	// Bubble sort
	// TODO: sorting go through memory, while it should be in registers
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		Samples[SampleIt] = (FOITSample)0;
		Samples[SampleIt].Depth = 10e28f;

		const uint2 SampleOffset = uint2(SampleIt % MaxSideSampleCount, SampleIt / MaxSideSampleCount);
		const uint2 SampleCoord  = SampleBaseCoord + SampleOffset;

		FOITSample Current = OITLoadSample(SampleDataTexture, SampleCoord);
		if (SampleIt > 0)
		{
			for (uint j = 0; j <= SampleIt; ++j)
			{
				if (Current.Depth < Samples[j].Depth) // Scene depth
				{
					OITSwap(Samples[j], Current);
				}
			}
		}
		else
		{
			Samples[SampleIt] = Current;
		}
	}

	// Combine all samples from front to back
	float3 FinalColor = 0;
	float3 Transmittance = 1;
	for (uint CompIt = 0; CompIt < SampleCount; ++CompIt)
	{
		FinalColor	 += Samples[CompIt].Color * Transmittance;
		Transmittance = saturate(Transmittance * Samples[CompIt].Trans);
	}

	// Alpha value used for separate translucency pass
	float AdjustedAlpha = saturate(dot(Transmittance, float3(1.0f, 1.0f, 1.0f) / 3.0f));

	// Write final color - typed load
	#if PERMUTATION_MODULATE
	if (SampleCount > 0)
	{
		OutLightingTexture[PixelCoord] = float4(FinalColor, PassType == OIT_PASS_SEPARATE ? AdjustedAlpha : OutLightingTexture[PixelCoord].a);
		OutTransmittanceTexture[PixelCoord] = float4(Transmittance, PassType == OIT_PASS_SEPARATE ? AdjustedAlpha : OutTransmittanceTexture[PixelCoord].a);
	}
	else
	{
		// If there is no translucent sample, clear the transmittance buffer to 1
		OutTransmittanceTexture[PixelCoord] = 1.f;
	}
	#else
	if (SampleCount > 0)
	{
		OutColorTexture[PixelCoord] = float4(FinalColor + OutColorTexture[PixelCoord].rgb * Transmittance, PassType == OIT_PASS_SEPARATE ? AdjustedAlpha : OutColorTexture[PixelCoord].a);
	}
	#endif
}
#endif // SHADER_OIT_COMBINE

///////////////////////////////////////////////////////////////////////////////////////////////////
// Debug

#if SHADER_OIT_DEBUG

#include "ShaderPrint.ush"

int2 CursorCoord;

void WriteEnable(inout FShaderPrintContext Context, bool bEnable)
{
	if (bEnable)
	{
		Print(Context, TEXT("On"), FontGreen);
	}
	else
	{
		Print(Context, TEXT("Off"), FontRed);
	}
}

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId == 0) && all(CursorCoord > 0) && all(CursorCoord < Resolution-1))
	{
		const uint2 PixelCoord	= CursorCoord;
		uint  SampleCount		= SampleCountTexture.Load(uint3(PixelCoord, 0));
		uint2 SampleBaseCoord	= PixelCoord * MaxSideSampleCount;
		
		// Global info
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 80));

			if (Method == 0)
			{
				Print(Context, TEXT("Method          : K-Buffer"), FontOrange);
			}
			else
			{
				Print(Context, TEXT("Method          : MLAB"), FontOrange);
			}
			Newline(Context);

			Print(Context, TEXT("Max Sample Count: "), FontWhite);
			Print(Context, MaxSampleCount, FontYellow);
			Newline(Context);

			Print(Context, TEXT("Current Pass    : "), FontWhite);
			if (!!(PassType & OIT_PASS_REGULAR))
			{
				Print(Context, TEXT("Regular Translucency"), FontYellow);
			}
			else if (!!(PassType & OIT_PASS_SEPARATE))
			{
				Print(Context, TEXT("Separate Translucency"), FontYellow);
			}
			Newline(Context);

			Print(Context, TEXT("Transp. Regular : "), FontWhite); WriteEnable(Context, (SupportedPass & OIT_PASS_REGULAR) > 0);
			Newline(Context);

			Print(Context, TEXT("Transp. Separate: "), FontWhite); WriteEnable(Context, (SupportedPass & OIT_PASS_SEPARATE) > 0);
			Newline(Context);
		}

		// Pixel coord
		FShaderPrintContext Context = InitShaderPrintContext(true, PixelCoord);

		Print(Context, TEXT("Pixel Coord.: "), FontWhite);
		Print(Context, PixelCoord, FontYellow);
		Newline(Context);

		FFontColor SampleCountColor = FontEmerald;
		if (SampleCount > MaxSampleCount)
		{
			SampleCountColor = FontRed;
		}
		Print(Context, TEXT("Sample Count: "), FontWhite);
		Print(Context, SampleCount, SampleCountColor);
		Print(Context, MaxSampleCount, SampleCountColor);
		Newline(Context);

		if (Method == 0)
		{
			Print(Context, TEXT("Method: K-Buffer"), FontOrange);
		}
		else
		{
			Print(Context, TEXT("Method: MLAB"), FontOrange);
		}
		Newline(Context);
		

		// Sample infos
		SampleCount = min(SampleCount, MaxSampleCount);

		for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			const uint2 SampleOffset = uint2(SampleIt % MaxSideSampleCount, SampleIt / MaxSideSampleCount);
			const uint2 SampleCoord = SampleBaseCoord + SampleOffset;

			const FOITSample Sample = OITLoadSample(SampleDataTexture, SampleCoord);

			Print(Context, TEXT("Depth "), FontYellow);
			Print(Context, Sample.Depth, FontYellow);
			Newline(Context);

			Print(Context, TEXT("Color "), FontEmerald);	
			Print(Context, Sample.Color, InitFontColor(Sample.Color));
			Newline(Context);

			Print(Context, TEXT("Trans."), FontOrange);	
			Print(Context, Sample.Trans, InitFontColor(Sample.Trans));
			Newline(Context);
		}
	}
}
#endif