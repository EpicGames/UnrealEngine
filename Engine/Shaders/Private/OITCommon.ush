// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef OIT_ENABLED
#define OIT_ENABLED 0
#endif

#ifndef OIT_IS_BASEPASS
#define OIT_IS_BASEPASS 0
#endif

#if OIT_ENABLED

//////////////////////////////////////////////////////////////////////////////////////////
// Sample storage format 

#define OIT_FORMAT_10BITS  1
#define OIT_FORMAT_16BITS  2 
#define OIT_FORMAT_COMPACT 3
#define OIT_FORMAT OIT_FORMAT_10BITS

//////////////////////////////////////////////////////////////////////////////////////////

struct FOITSample
{
	float3 Color;
	float  Depth;
	float3 Trans;
	float  Pad;
};

//////////////////////////////////////////////////////////////////////////////////////////

uint OITPack10F(float Value)
{
	return (f32tof16(Value) >> 5) & 0x000003FF;
}

float OITUnpack10F(uint Value)
{
	return f16tof32((Value << 5) & 0x7FE0);
}

uint OITPack6F(float Value)
{
	return (f32tof16(Value) >> 5) & 0x000003FF;
}

float OITUnpack6F(uint Value)
{
	return f16tof32((Value << 5) & 0x7FE0);
}


void OITSwap(inout FOITSample A, inout FOITSample B)
{
	FOITSample T = A;
	A = B;
	B = T;
}

uint OITPackR11G11B10F(float3 rgb)
{
	uint r = (f32tof16(rgb.r) << 17) & 0xFFE00000;
	uint g = (f32tof16(rgb.g) << 6) & 0x001FFC00;
	uint b = (f32tof16(rgb.b) >> 5) & 0x000003FF;
	return r | g | b;
}

float3 OITUnpackR11G11B10F(uint rgb)
{
	float r = f16tof32((rgb >> 17) & 0x7FF0);
	float g = f16tof32((rgb >> 6) & 0x7FF0);
	float b = f16tof32((rgb << 5) & 0x7FE0);
	return float3(r, g, b);
}

//////////////////////////////////////////////////////////////////////////////////////////
// Per-attribut compaction 12bytes/96bits

#if OIT_FORMAT == OIT_FORMAT_10BITS

#define FOITPackedSample uint3
#define OIT_DATA_COUNT 3

FOITPackedSample PackOITSample(FOITSample In)
{
	FOITPackedSample Out = (FOITPackedSample)0;
	Out.x = asuint(In.Depth);
	Out.y = OITPackR11G11B10F(In.Color);
	Out.z = OITPackR11G11B10F(saturate(In.Trans));
	return Out;
}

FOITSample UnpackOITSample(FOITPackedSample In)
{
	FOITSample Out = (FOITSample)0;
	Out.Depth = asfloat(In.x);
	Out.Color = OITUnpackR11G11B10F(In.y);
	Out.Trans = saturate(OITUnpackR11G11B10F(In.z));
	return Out;
}
#endif

//////////////////////////////////////////////////////////////////////////////////////////
// Fat compaction 16bytes/128bits

#if OIT_FORMAT == OIT_FORMAT_16BITS

#define FOITPackedSample uint4
#define OIT_DATA_COUNT 4

FOITPackedSample PackOITSample(float3 InColor, float3 InTrans, float InDepth)
{
	uint4 Out = 0;
	Out.x = f32tof16(InColor.r) | (f32tof16(InColor.g) << 16);
	Out.y = f32tof16(InColor.b) | (f32tof16(InTrans.r) << 16);
	Out.z = f32tof16(InTrans.g) | (f32tof16(InTrans.b) << 16);
	Out.w = InDepth;
	return Out;
}

void UnpackOITSample(FOITPackedSample In, inout float3 OutColor, inout float3 OutTrans, inout float OutDepth)
{
	OutColor.r = f16tof32(0xFFFF & (In.x));
	OutColor.g = f16tof32(0xFFFF & (In.x >> 16));
	OutColor.b = f16tof32(0xFFFF & (In.y));

	OutTrans.r = f16tof32(0xFFFF & (In.y >> 16));
	OutTrans.g = f16tof32(0xFFFF & (In.z));
	OutTrans.b = f16tof32(0xFFFF & (In.z >> 16));

	OutDepth = In.w;
}
#endif // OIT_FORMAT_16BITS

//////////////////////////////////////////////////////////////////////////////////////////
// Compact sample into 8bytes/64bits (untested|add dithering if banding)

#if OIT_FORMAT == OIT_FORMAT_COMPACT

#define FOITPackedSample uint2
#define OIT_DATA_COUNT 2

FOITPackedSample PackOITSample(float3 InColor, float3 InTrans, float InDepth)
{
	uint ColorR10 = OITPack10F(InColor.r);
	uint ColorG10 = OITPack10F(InColor.g);
	uint ColorB10 = OITPack10F(InColor.b);

	uint TransR6 = OITPack6F(InTrans.r);
	uint TransG6 = OITPack6F(InTrans.g);
	uint TransB6 = OITPack6F(InTrans.b);

	uint Depth16 = f32tof16(InDepth);
	
	FOITPackedSample Out = 0;
	Out.x = ColorR10 | (ColorG10 << 10) | (ColorB10 << 20) | (((TransB6 >> 4) & 0x3) << 30);
	Out.y = Depth16  | (TransR6  << 16) | (TransG6  << 22) | (TransB6 & 0xF) << 28);
	return Out;
}

void UnpackOITSample(FOITPackedSample In, inout float3 OutColor, inout float3 OutTrans, inout float OutDepth)
{
	OutColor.r = OITUnpack10F((In.x)     & 3FF);
	OutColor.g = OITUnpack10F((In.x>>10) & 3FF);
	OutColor.b = OITUnpack10F((In.x>>20) & 3FF);

	OutTrans.r = OITPack6F( (In.y>>16) & 0x3F);
	OutTrans.g = OITPack6F( (In.y>>22) & 0x3F);
	OutTrans.b = OITPack6F(((In.y>>28) & 0xF) | ((In.x>>30) & 0x3));

	OutDepth = f16tof32(In.y & 0xFFFF);
}
#endif // OIT_FORMAT_COMPACT

//////////////////////////////////////////////////////////////////////////////////////////

FOITSample OITLoadSample(Texture2DArray<uint> InTexture, uint2 InPixelCoord)
{
	FOITPackedSample Packed = (FOITPackedSample)0;
	Packed.x = InTexture[uint3(InPixelCoord, 0)];
#if OIT_DATA_COUNT > 1
	Packed.y = InTexture[uint3(InPixelCoord, 1)];
#endif
#if OIT_DATA_COUNT > 2
	Packed.z = InTexture[uint3(InPixelCoord, 2)];
#endif
#if OIT_DATA_COUNT > 3
	Packed.w = InTexture[uint3(InPixelCoord, 3)];
#endif
	return UnpackOITSample(Packed);
}

#if OIT_IS_BASEPASS

//////////////////////////////////////////////////////////////////////////////////////////
// Special Add/Load/Store sample function for base-pass

FOITSample OITLoadSample(uint2 InPixelCoord)
{
	FOITPackedSample Packed = (FOITPackedSample)0;
	Packed.x = TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 0)];
#if OIT_DATA_COUNT > 1
	Packed.y = TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 1)];
#endif
#if OIT_DATA_COUNT > 2
	Packed.z = TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 2)];
#endif
#if OIT_DATA_COUNT > 3
	Packed.w = TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 3)];
#endif
	return UnpackOITSample(Packed);
}

void OITStoreSample(uint2 InPixelCoord, FOITSample In)
{
	FOITPackedSample Packed = PackOITSample(In);
	TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 0)] = Packed.x;
#if OIT_DATA_COUNT > 1
	TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 1)] = Packed.y;
#endif
#if OIT_DATA_COUNT > 2
	TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 2)] = Packed.z;
#endif
#if OIT_DATA_COUNT > 3
	TranslucentBasePass.OIT.OutOITSampleData[uint3(InPixelCoord, 4)] = Packed.w;
#endif
}

// Simple k-buffer insertion
void AddOITSample_KB(uint2 InPixelCoord, float3 InColor, float3 InTrans, float InDepth)
{
	uint SampleOffset = 0;
	InterlockedAdd(TranslucentBasePass.OIT.OutOITSampleCount[InPixelCoord], 1u, SampleOffset);

	if (SampleOffset < TranslucentBasePass.OIT.MaxSamplePerPixel)
	{
		const uint2 SampleLocalCoord = uint2(SampleOffset % TranslucentBasePass.OIT.MaxSideSamplePerPixel, SampleOffset / TranslucentBasePass.OIT.MaxSideSamplePerPixel);
		const uint2 SampleCoord = InPixelCoord * TranslucentBasePass.OIT.MaxSideSamplePerPixel + SampleLocalCoord;

		FOITSample Sample = (FOITSample)0;
		Sample.Color = InColor;
		Sample.Trans = InTrans;
		Sample.Depth = InDepth;

		OITStoreSample(SampleCoord, Sample);
	}
}

uint2 GetStoringBaseCoord(uint2 InPixelCoord)
{
	return InPixelCoord * TranslucentBasePass.OIT.MaxSideSamplePerPixel;
}

uint2 GetStoringCoord(uint2 InBaseCoord, uint InSampleIt)
{
	const uint2 SampleOffset = uint2(InSampleIt % TranslucentBasePass.OIT.MaxSideSamplePerPixel, InSampleIt / TranslucentBasePass.OIT.MaxSideSamplePerPixel);
	return InBaseCoord + SampleOffset;
}

// Multi-Layer Alpha Blending (MLAB) - Sample are inserted in a sorted manner and overflow sample are merged with alpha blending
void AddOITSample_MLAB(uint2 InPixelCoord, float3 InColor, float3 InTrans, float InDepth)
{
	RASTER_ORDERED_VIEW_LOCK();

	// ROV read to for pixel synchronization
	uint SampleCount = min(TranslucentBasePass.OIT.OutOITSampleCount[InPixelCoord], TranslucentBasePass.OIT.MaxSamplePerPixel);

	FOITSample Current = (FOITSample)0;
	Current.Depth = InDepth;
	Current.Color = InColor;
	Current.Trans = InTrans;

	const uint2 SampleBaseCoord = GetStoringBaseCoord(InPixelCoord);

	// Check if sample fall into the merged/combined bucket, and if it needs to be discard due to transmission threshold
	bool bNeedInsertSort = true;
	if (SampleCount > 0)
	{
		const uint2 LastSampleCoord = GetStoringCoord(SampleBaseCoord, SampleCount-1);
		const FOITSample LastSample = OITLoadSample(LastSampleCoord);
		if (Current.Depth > LastSample.Depth) // Scene depth (not inv-Z)
		{
			if (all(LastSample.Trans < TranslucentBasePass.OIT.TransmittanceThreshold))
			{
				// The return is import here as it avoid GPU hang. Discard is not enough here, probably due to ROV/UAV output
				discard;
				return;
			}
		}
	}

	// Bubble sort and insert
	if (bNeedInsertSort)
	{
		for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
		{
			const uint2 SampleCoord = GetStoringCoord(SampleBaseCoord, SampleIt);
			const FOITSample Sample = OITLoadSample(SampleCoord);

			if (Current.Depth < Sample.Depth) // Scene depth (not inv-Z)
			{
				OITStoreSample(SampleCoord, Current);
				Current = Sample;
			}			
		}
	}

	// Merge last fragment
	if (SampleCount >= TranslucentBasePass.OIT.MaxSamplePerPixel)
	{
		const uint2 SampleCoord = GetStoringCoord(SampleBaseCoord, TranslucentBasePass.OIT.MaxSamplePerPixel - 1);

		FOITSample Last = OITLoadSample(SampleCoord);
		Last.Color = Last.Color + Last.Trans * Current.Color;
		Last.Trans = saturate(Last.Trans * Current.Trans);
		Last.Depth = min(Last.Depth, Current.Depth); // Normally Last.Depth should already be the min. depth

		OITStoreSample(SampleCoord, Last);
	}
	else
	{
		// If there are still room to store sample (i.e., not over budget), insert the sample at the end (note: SampleCount has not being incremented yet)
		const uint2 SampleCoord = GetStoringCoord(SampleBaseCoord, SampleCount);
		OITStoreSample(SampleCoord, Current);
	}

	// Write to ROV to unlock pixel
	SampleCount = min(SampleCount + 1, TranslucentBasePass.OIT.MaxSamplePerPixel);
	TranslucentBasePass.OIT.OutOITSampleCount[InPixelCoord] = SampleCount;

	RASTER_ORDERED_VIEW_UNLOCK();
}

// Add a translucent sample to the list of existing translucent samples
// InColor : Radiance/Luminance reflected by the surface (pre-multiplied alpha)
// InTrans : Transmittance of the surface (i.e., 1-Opacity)
// InDepth : Scene depth (i.e. not DeviceZ)
void AddOITSample(uint2 InPixelCoord, float3 InColor, float3 InTrans, float InDepth)
{
	AddOITSample_MLAB(InPixelCoord, InColor, InTrans, InDepth);
}

//////////////////////////////////////////////////////////////////////////////////////////

#endif // OIT_IS_BASEPASS

#endif // OIT_ENABLED