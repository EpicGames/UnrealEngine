// Copyright Epic Games, Inc. All Rights Reserved.
#include "Common.ush"

#if SHADING_PATH_MOBILE 
	#define OcclusionBuffer MobileBasePass.RWOcclusionBufferUAV
#else
	#define OcclusionBuffer DummyBuffer
#endif

uint StartIndex;
#if INSTANCED_STEREO
uint InstancedViewIndex;
#endif

void MainVS(
	float4 InPosition	: ATTRIBUTE0,
	float4 InOrigin		: ATTRIBUTE1,
	float4 InExtent		: ATTRIBUTE2,
	uint InstanceId		: SV_InstanceID,
#if MOBILE_MULTI_VIEW && !INSTANCED_STEREO
	uint ViewId			: SV_ViewID,
#endif
	out nointerpolation uint PrimitiveIdx : PRIMITIVE_ID,
	out float4 OutPosition : SV_POSITION
#if MOBILE_MULTI_VIEW && INSTANCED_STEREO
	, out uint StereoLayer : SV_RenderTargetArrayIndex
#elif INSTANCED_STEREO
	, out uint StereoLayer : SV_ViewPortArrayIndex
#endif
)
{
#if MOBILE_MULTI_VIEW && !INSTANCED_STEREO
	ResolvedView = ResolveView(ViewId);
#elif INSTANCED_STEREO
	uint ViewId = InstancedViewIndex;
	StereoLayer = ViewId;
	ResolvedView = ResolveView(ViewId);
#else
	ResolvedView = ResolveView();
#endif

	float3 TransformedPosition = (InPosition.xyz * InExtent.xyz) + InOrigin.xyz;
	OutPosition = mul(float4(TransformedPosition, 1), ResolvedView.TranslatedWorldToClip);
	PrimitiveIdx = StartIndex + InstanceId;
}

EARLYDEPTHSTENCIL
void MainPS(
	in nointerpolation uint PrimitiveIdx : PRIMITIVE_ID,
	out float4 OutColor : SV_Target0
)
{
	// TODO: write only some pixels? 
	OcclusionBuffer[PrimitiveIdx] = 1u;

#if PERMUTATION_PIXEL_DISCARD
	clip(-1.0f);
#endif
	OutColor = float4(1, 0, 0, 1);
}