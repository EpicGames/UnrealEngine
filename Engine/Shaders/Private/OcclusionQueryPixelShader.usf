// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OcclusionQueryPixelShader.usf: Pixel shader for visualizing occlusion queries.
=============================================================================*/

#include "Common.ush"

void Main(out float4 OutColor : SV_Target0)
{
	OutColor = float4(1, 0, 0, 1);
}
