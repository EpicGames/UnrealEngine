// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OcclusionQueryVertexShader.usf: Vertex shader for drawing occlusion queries.
=============================================================================*/

#include "Common.ush"

#if INSTANCED_STEREO || MOBILE_MULTI_VIEW
float4 StencilingGeometryPosAndScale[2];
#else
float4 StencilingGeometryPosAndScale;
#endif

void Main(
	in float4 InPosition : ATTRIBUTE0,
	out float4 OutPosition : SV_POSITION
#if INSTANCED_STEREO
	, uint InstanceId : SV_InstanceID
#if MOBILE_MULTI_VIEW
	, out nointerpolation uint LayerIndex : SV_RenderTargetArrayIndex
#else
	, out nointerpolation uint ViewportIndex : SV_ViewPortArrayIndex
#endif // MOBILE_MULTI_VIEW
#elif MOBILE_MULTI_VIEW
	, in uint InViewId : SV_ViewID
#endif // INSTANCED_STEREO
	)
{
#if INSTANCED_STEREO
	uint InViewId = GetEyeIndex(InstanceId);
#if MOBILE_MULTI_VIEW
	LayerIndex = InViewId;
#else
	ViewportIndex = InViewId;
#endif
#endif
#if INSTANCED_STEREO || MOBILE_MULTI_VIEW
	ResolvedView = ResolveView(InViewId);
	float4 PosAndScale = StencilingGeometryPosAndScale[InViewId];
#else
	ResolvedView = ResolveView();
	float4 PosAndScale = StencilingGeometryPosAndScale;
#endif

	float3 TransformedPosition = InPosition.xyz * PosAndScale.w + PosAndScale.xyz;
	OutPosition = mul(float4(TransformedPosition, 1), ResolvedView.TranslatedWorldToClip);
}
