// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OctahedralCommon.ush
=============================================================================*/

#pragma once

// Octahedron Normal Vectors
// [Cigolle 2014, "A Survey of Efficient Representations for Independent Unit Vectors"]
//						Mean	Max
// oct		8:8			0.33709 0.94424
// snorm	8:8:8		0.17015 0.38588
// oct		10:10		0.08380 0.23467
// snorm	10:10:10	0.04228 0.09598
// oct		12:12		0.02091 0.05874

float2 UnitVectorToOctahedron( float3 N )
{
	N.xy /= dot( 1, abs(N) );
	if( N.z <= 0 )
	{
		N.xy = ( 1 - abs(N.yx) ) * select( N.xy >= 0, float2(1,1), float2(-1,-1) );
	}
	return N.xy;
}

float3 OctahedronToUnitVector( float2 Oct )
{
	float3 N = float3( Oct, 1 - dot( 1, abs(Oct) ) );
	float t = max( -N.z, 0 );
	N.xy += select(N.xy >= 0, float2(-t, -t), float2(t, t));
	return normalize(N);
}

float2 UnitVectorToHemiOctahedron( float3 N )
{
	N.xy /= dot( 1, abs(N) );
	return float2( N.x + N.y, N.x - N.y );
}

float3 HemiOctahedronToUnitVector( float2 Oct )
{
	Oct = float2( Oct.x + Oct.y, Oct.x - Oct.y );
	float3 N = float3( Oct, 2.0 - dot( 1, abs(Oct) ) );
	return normalize(N);
}

// Wrap around octahedral map for correct hardware bilinear filtering
uint2 OctahedralMapWrapBorder(uint2 TexelCoord, uint Resolution, uint BorderSize)
{
	if (TexelCoord.x < BorderSize)
	{
		TexelCoord.x = BorderSize - 1 + BorderSize - TexelCoord.x;
		TexelCoord.y = Resolution - 1 - TexelCoord.y;
	}
	if (TexelCoord.x >= Resolution - BorderSize)
	{
		TexelCoord.x = (Resolution - BorderSize) - (TexelCoord.x - (Resolution - BorderSize - 1));
		TexelCoord.y = Resolution - 1 - TexelCoord.y;
	}
	if (TexelCoord.y < BorderSize)
	{
		TexelCoord.y = BorderSize - 1 + BorderSize - TexelCoord.y;
		TexelCoord.x = Resolution - 1 - TexelCoord.x;
	}
	if (TexelCoord.y >= Resolution - BorderSize)
	{
		TexelCoord.y = (Resolution - BorderSize) - (TexelCoord.y - (Resolution - BorderSize - 1));
		TexelCoord.x = Resolution - 1 - TexelCoord.x;
	}

	return TexelCoord - BorderSize;
}

// Computes the spherical excess (solid angle) of a spherical triangle with vertices A, B, C as unit length vectors
// https://en.wikipedia.org/wiki/Spherical_trigonometry#Area_and_spherical_excess
float ComputeSphericalExcess(float3 A, float3 B, float3 C) {
    float CosAB = dot(A, B);
    float SinAB = 1.0f - CosAB * CosAB;
    float CosBC = dot(B, C);
    float SinBC = 1.0f - CosBC * CosBC;
    float CosCA = dot(C, A);
    float CosC = CosCA - CosAB * CosBC;
    float SinC = sqrt(SinAB * SinBC - CosC * CosC);
    float Inv = (1.0f - CosAB) * (1.0f - CosBC);
	return 2.0f * atan2(SinC, sqrt((SinAB * SinBC * (1.0f + CosBC) * (1.0f + CosAB)) / Inv) + CosC);
}

// TexelCoord should be centered on the octahedral texel, in the range [.5f, .5f + Resolution - 1]
float OctahedralSolidAngle(float2 TexelCoord, float InvResolution)
{
	float3 Direction10 = OctahedronToUnitVector((TexelCoord + float2(.5f, -.5f) * InvResolution) * 2.0f - 1.0f);
	float3 Direction01 = OctahedronToUnitVector((TexelCoord + float2(-.5f, .5f) * InvResolution) * 2.0f - 1.0f);

	float SolidAngle0 = ComputeSphericalExcess(
		OctahedronToUnitVector((TexelCoord + float2(-.5f, -.5f) * InvResolution) * 2.0f - 1.0f), 
		Direction10,
		Direction01);

	float SolidAngle1 = ComputeSphericalExcess(
		OctahedronToUnitVector((TexelCoord + float2(.5f, .5f) * InvResolution) * 2.0f - 1.0f), 
		Direction01,
		Direction10);

	return SolidAngle0 + SolidAngle1;
}