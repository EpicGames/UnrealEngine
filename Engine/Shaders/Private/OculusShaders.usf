// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

void MainVertexShader(
	float4 InPosition : ATTRIBUTE0,
	float2 InUV : ATTRIBUTE1,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition = InPosition;
	OutUV = InUV;
}

Texture2D<uint> TextureParameter;
 
void MainWhiteShader(
	out float4 OutColor : SV_Target0
	)
{
	OutColor = float4(1, 1, 1, 1);
}

void MainBlackShader(
	out float4 OutColor : SV_Target0
	)
{
	OutColor = float4(0, 0, 0, 0);
}

Texture2D InTexture;
TextureCube InTextureCube;
SamplerState InTextureSampler;

void MainAlphaInverseShader(
	in float2 uv : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	float InverseAlpha = 1 - Texture2DSample(InTexture, InTextureSampler, uv).a;
	OutColor = float4(0, 0, 0, InverseAlpha);
}

int CubeFaceIndex;

void MainForCubemap(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	float u = Input.UV.x * 2. - 1.0;
	float v = Input.UV.y * 2. - 1.0;

	if(CubeFaceIndex == 0)
	{
	    OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(1., v, u));
	}
	else if(CubeFaceIndex == 1)
	{
		OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(-1., v, -u));
	}
	else if(CubeFaceIndex == 2)
	{
		OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(u, 1., v));
	}
	else if(CubeFaceIndex == 3)
	{
		OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(u, -1., -v));
	}
	else if(CubeFaceIndex == 5)
	{
		OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(-u,v,1.));
	}
	else if(CubeFaceIndex == 4)
	{
		OutColor = TextureCubeSample(InTextureCube, InTextureSampler, float3(u,v,-1.));
	}
}
