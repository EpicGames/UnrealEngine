// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	OneColorShader.usf: 2D shader for drawing a single constant color.
=============================================================================*/

#include "Common.ush"



/*=============================================================================
 * Vertex shader
 =============================================================================*/

struct FOneColorVertexOutput
{
	float4 Position : SV_POSITION;
#if USING_VERTEX_SHADER_LAYER && USING_LAYERS
	uint LayerIndex : SV_RenderTargetArrayIndex;
#endif
};

float InputDepth;

#ifndef USING_NDC_POSITIONS
	#define USING_NDC_POSITIONS 1
#endif

void MainVertexShader(
	in float4 InPosition : ATTRIBUTE0,
#if USING_VERTEX_SHADER_LAYER && USING_LAYERS
	in uint InstanceId	: SV_InstanceID,
#endif
	out FOneColorVertexOutput Output
	)
{
	float4 Position = float4(InPosition.xy, InputDepth, 1.0f);
	Output.Position = Position;

#if !USING_NDC_POSITIONS
	DrawRectangle(Position, Output.Position);
#endif

#if USING_VERTEX_SHADER_LAYER && USING_LAYERS
	Output.LayerIndex = InstanceId;
#endif
}

/*=============================================================================
 * Pixel shader
 =============================================================================*/

#ifndef NUM_OUTPUTS
#define NUM_OUTPUTS 1
#endif
#ifndef NUM_UINT_OUTPUTS
#define NUM_UINT_OUTPUTS 0
#endif

#define NUM_FLOAT_OUTPUTS (NUM_OUTPUTS - NUM_UINT_OUTPUTS) 

float4 DrawColorMRT[8];

//These shaders used for RHIClear.  Don't use any global uniform variables here as it greatly complicates
//the state restoration for an inline shader clear.
void MainPixelShaderMRT( 
#if NUM_UINT_OUTPUTS == 0
	out float4	OutColor[NUM_OUTPUTS]					: SV_Target0
#elif NUM_UINT_OUTPUTS == NUM_OUTPUTS
	out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target0
#else
	out float4	OutColor[NUM_OUTPUTS - NUM_UINT_OUTPUTS]: SV_Target0
#if NUM_FLOAT_OUTPUTS == 1
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target1
#elif(NUM_FLOAT_OUTPUTS == 2)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target2
#elif(NUM_FLOAT_OUTPUTS == 3)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target3
#elif(NUM_FLOAT_OUTPUTS == 4)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target4
#elif(NUM_FLOAT_OUTPUTS == 5)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target5
#elif(NUM_FLOAT_OUTPUTS == 6)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target6
#elif(NUM_FLOAT_OUTPUTS == 7)
	, out uint	OutUints[NUM_UINT_OUTPUTS]				: SV_Target7
#endif
#endif
)
{
#if NUM_UINT_OUTPUTS < NUM_OUTPUTS
	for (int i0 = 0; i0 < NUM_FLOAT_OUTPUTS; i0++)
	{
		OutColor[i0] = DrawColorMRT[i0];
	}
#endif
#if NUM_UINT_OUTPUTS > 0
	for (int i1 = 0; i1 < NUM_UINT_OUTPUTS; i1++)
	{
		OutUints[i1] = DrawColorMRT[NUM_FLOAT_OUTPUTS + i1].x;
	}
#endif
}

void MainPixelShader( 
	out float4 OutColor : SV_Target0)
{
	OutColor = DrawColorMRT[0];
}

/** 
 * Basic unit of a long GPU task
 * To make sure that we're not CPU bound when profiling the GPU, as we can't isolate GPU idle time on PC. 
 * Measured execution time on a 1080p render target:
 * Intel HD Graphics 620: ~50ms
 * NVIDIA GeForce GTX 670: ~8.02ms
 * NVIDIA GeForce GTX 980: ~4.00ms
 * NVIDIA GeForce GTX 1080: ~2.11ms
 */
void MainLongGPUTask(
	FOneColorVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = 0;

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	for (int i = 0; i < 1200; i++)
	{
		OutColor += .0001f * cos(Input.Position.x) * sin(Input.Position.y);
	}
#else
	// Not effective, deprecated
	for (int i = 0; i < 255; i++)
	{
		OutColor += .0001f * cos(i * .0001f) * sin(i * i * .00001f);
	}
#endif

	OutColor *= .000001f;
}

float4 FillValue;
RWTexture2D<float4> FillTexture;
float4 Params0; // Texture Width,Height (.xy); Use Exclude Rect 1 : 0 (.z)
float4 Params1; // Include X0,Y0 (.xy) - X1,Y1 (.zw)
float4 Params2; // ExcludeRect X0,Y0 (.xy) - X1,Y1 (.zw)

[numthreads(8,8,1)]
void MainFillTextureCS(uint2 XY : SV_DispatchThreadID)
{
	float Width = Params0.x;
	float Height = Params0.y;
	bool bUseExcludeRect = (Params0.z != 0);

	float X = XY.x;
	float Y = XY.y;
	float IncludeX0 = Params1.x;
	float IncludeY0 = Params1.y;
	float IncludeX1 = Params1.z;
	float IncludeY1 = Params1.w;

	if (X < Width && Y < Height)
	{
		if (X >= IncludeX0 && X <= IncludeX1 && Y >= IncludeY0 && Y <= IncludeY1)
		{
			bool bDoWrite = true;
/*
			if (bUseExcludeRect)
			{
				float ExcludeX0 = Params2.x;
				float ExcludeY0 = Params2.y;
				float ExcludeX1 = Params2.z;
				float ExcludeY1 = Params2.w;
				if (X >= ExcludeX0 && X <= ExcludeX1 && Y >= ExcludeY0 && Y <= ExcludeY1)
				{
					bDoWrite = false;
				}
			}
*/

			if (bDoWrite)
			{
				FillTexture[XY] = FillValue;
			}
		}
	}
}
