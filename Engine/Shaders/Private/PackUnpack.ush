// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackUnpack.ush: Common shader code for packing and unpacking data. 
					Automatically included from Common.ush.
=============================================================================*/

#pragma once

#include "/Engine/Public/Platform.ush"

//
//	float2 <-> u32 as Unorm16.Unorm16
//

uint PackUnorm2x16(float2 v)
{
	uint2 sv = uint2(round(clamp(v, 0.0, 1.0) * 65535.0));
	return (sv.x | (sv.y << 16u));
}

float2 UnpackUnorm2x16(uint p)
{
	float2 Ret;
	Ret.x = (p & 0xffff) * rcp(65535.0f);
	Ret.y = (p >> 16u) * rcp(65535.0f);
	return Ret;
}

//
//	float2 <-> u32 as Snorm16.Snorm16 
//

uint PackSnorm2x16(float2 v)
{
	uint2 sv = uint2(round(clamp(v, -1.0, 1.0) * 32767.0) + 32767.0);
	return (sv.x | (sv.y << 16u));
}

float2 UnpackSnorm2x16(uint p)
{
	float2 Ret;
	Ret.x = clamp((float(p & 0xffff) - 32767.0f) * rcp(32767.0f), -1.0, 1.0);
	Ret.y = clamp((float(p >> 16u) - 32767.0f) * rcp(32767.0f), -1.0, 1.0);
	return Ret;
}

//
//	uint2 <-> u32 as u16.u16
//

#ifndef COMPILER_SUPPORTS_PACK_INTRINSICS
#define COMPILER_SUPPORTS_PACK_INTRINSICS 0

uint PackUInt2ToUInt(uint X, uint Y)
{
	return X | (Y << 16u);
}

#endif

uint PackUInt2ToUInt(uint2 XY)
{
	return PackUInt2ToUInt(XY.x, XY.y);
}

uint2 UnpackUInt2FromUInt(uint Packed)
{
	return uint2(Packed & 0xffff, Packed >> 16);
}

//
//	float2 <-> u32 as f16.f16
//

#if !COMPILER_SUPPORTS_PACK_INTRINSICS
uint PackFloat2ToUInt(float X, float Y)
{
	return PackUInt2ToUInt(f32tof16(X), f32tof16(Y));
}
#endif

uint PackFloat2ToUInt(float2 XY)
{
	return PackFloat2ToUInt(XY.x, XY.y);
}

float2 UnpackFloat2FromUInt(uint In)
{
	return float2(f16tof32(In), f16tof32(In >> 16));
}

//
//	float <-> u8
//

uint PackR8(float Value)
{
	return uint(saturate(Value) * 255.0f);
}

float UnpackR8(uint In)
{
#if COMPILER_SUPPORTS_UNPACKBYTEN
	return UnpackByte0(In) * (1.0f / 255.0f);
#else
	return float(BitFieldExtractU32(In, 8, 0)) * (1.0f / 255.0f);
#endif
}

float UnpackG8(uint In)
{
	#if COMPILER_SUPPORTS_UNPACKBYTEN
	return float(UnpackByte1(In)) * (1.0f / 255.0f);
	#else
	return float(BitFieldExtractU32(In, 8, 8)) * (1.0f / 255.0f);
	#endif
}

float UnpackB8(uint In)
{
	#if COMPILER_SUPPORTS_UNPACKBYTEN
	return float(UnpackByte2(In)) * (1.0f / 255.0f);
	#else
	return float(BitFieldExtractU32(In, 8, 16)) * (1.0f / 255.0f);
	#endif
}

float UnpackA8(uint In)
{
	#if COMPILER_SUPPORTS_UNPACKBYTEN
	return float(UnpackByte3(In)) * (1.0f / 255.0f);
	#else
	return float(BitFieldExtractU32(In, 8, 24)) * (1.0f / 255.0f);
	#endif
}

//
//	float <-> u6
//

uint PackR6(float Value)
{
	return uint(saturate(Value) * 63.0f);
}

float UnpackR6(uint In)
{
	return float(In & 0x3F) * (1.0f / 63.0f);
}

//
//	float <-> u5
//

uint PackR5(float Value)
{
	return uint(saturate(Value) * 31.0f);
}

float UnpackR5(uint In)
{
	return float(In & 0x1F) * (1.0f / 31.0f);
}

//
//	float4 <-> u32 as u8.u8.u8.u8
//

uint PackRGBA8(float4 In)
{
#if COMPILER_SUPPORTS_PACK_INTRINSICS
	return PackFloat4ToUInt(saturate(In) * 255.0f);
#else
	uint r = (uint(saturate(In.r) * 255.0f) << 0);
	uint g = (uint(saturate(In.g) * 255.0f) << 8);
	uint b = (uint(saturate(In.b) * 255.0f) << 16);
	uint a = (uint(saturate(In.a) * 255.0f) << 24);
	return r | g | b | a;
#endif
}

float4 UnpackRGBA8(uint In)
{
#if COMPILER_SUPPORTS_UNPACKBYTEN
	return float4(
		UnpackByte0(In),
		UnpackByte1(In),
		UnpackByte2(In),
		UnpackByte3(In)) * (1.0f / 255.0f);
#else
	float4 Out;
	Out.r = float((In >> 0) & 0xFF) * (1.0f / 255.0f);
	Out.g = float((In >> 8) & 0xFF) * (1.0f / 255.0f);
	Out.b = float((In >> 16) & 0xFF) * (1.0f / 255.0f);
	Out.a = float((In >> 24) & 0xFF) * (1.0f / 255.0f);
	return Out;
#endif
}

uint PackUint4ToUint(uint4 In)
{
	// TODO use BitAlignU32
	uint r = (In.r & 0xFFu);
	uint g = (In.g & 0xFFu) << 8u;
	uint b = (In.b & 0xFFu) << 16u;
	uint a = (In.a & 0xFFu) << 24u;
	return r | g | b | a;
}

uint4 UnpackUintToUint4(uint In)
{
	// TODO use BitAlignU32
	uint4 Out;
	Out.r = (In >> 0u)  & 0xFFu;
	Out.g = (In >> 8u)  & 0xFFu;
	Out.b = (In >> 16u) & 0xFFu;
	Out.a = (In >> 24u) & 0xFFu;
	return Out;
}

//
//	float <-> unorm
//

uint PackUnorm10(float Value)
{
	return uint(saturate(Value) * 1023.0f);
}

float UnpackUnorm10(uint In)
{
	return float(In & 0x3FF) * (1.0f / 1023.0f);
}

//
//	float <-> f10
//

uint Pack10F(float Value)
{
	return (f32tof16(Value) >> 5) & 0x000003FF;
}

float Unpack10F(uint Value)
{
	return f16tof32((Value << 5) & 0x7FE0);
}

//
//	float3 <-> u32 as f11.f11.f10
//

uint PackR11G11B10F(float3 rgb)
{
	uint r = (f32tof16(rgb.r) << 17) & 0xFFE00000;
	uint g = (f32tof16(rgb.g) << 6) & 0x001FFC00;
	uint b = (f32tof16(rgb.b) >> 5) & 0x000003FF;
	return r | g | b;
}

float3 UnpackR11G11B10F(uint rgb)
{
	float r = f16tof32((rgb >> 17) & 0x7FF0);
	float g = f16tof32((rgb >> 6) & 0x7FF0);
	float b = f16tof32((rgb << 5) & 0x7FE0);
	return float3(r, g, b);
}

//
//	float3 <-> f10.f10.f10
//

uint PackR10G10B10F(float3 rgb)
{
	uint r = (f32tof16(rgb.r) << 15) & 0x3FF00000;	// 0011 1111 1111 0000 0000 0000 0000 0000 
	uint g = (f32tof16(rgb.g) << 5) & 0x000FFC00;	// 0000 0000 0000 1111 1111 1100 0000 0000 
	uint b = (f32tof16(rgb.b) >> 5) & 0x000003FF;	// 0000 0000 0000 0000 0000 0011 1111 1111
	return r | g | b;
}

float3 UnpackR10G10B10F(uint rgb)
{
	float r = f16tof32((rgb >> 15) & 0x7FE0);
	float g = f16tof32((rgb >> 5) & 0x7FE0);
	float b = f16tof32((rgb << 5) & 0x7FE0);
	return float3(r, g, b);
}

//
// float3 <-> uint
//  using a RGBE encoding with 9,9,8 bits for RGB and 6 bits for the shared exponent
// See a discussion of such encodings here: 
//   https://cbloomrants.blogspot.com/2020/06/widespread-error-in-radiance-hdr-rgbe.html
//   https://cbloomrants.blogspot.com/2020/06/followup-tidbits-on-rgbe.html 
// Because we are not using these functions in the context of Radiance .hdr files,
// we can deviate from the classic 888e8 format and apply the corrections suggested in
// the post above, while changing the bit depths to get a bit more color resolution at
// the expense of less dynamic range (exponent of 8 being excessive for colors anyway)
// The RGBE_plus encoding suggested in the second blog post is slightly better in some
// cases, but fairly close to the version below. This code has the advantage of being
// slightly easier to decode.

uint PackRGB998E6(float3 rgb)
{
	const float MAX_C = float((1 << 8) - 1) * asfloat((63 - 32 + 127) << 23);
	uint3 cbits = asuint(min(rgb, MAX_C));
	int3  exp = int3(cbits >> 23) - 127;
	uint3 man = (cbits & ((1 << 23) - 1)) | (1 << 23);

	int max_exp = max3(exp.x, exp.y, exp.z);

	// change from [1,2) to [0.5,1)
	max_exp++;
	max_exp = max(max_exp, -32);

	int3 shift = min(max_exp - exp - int3(9, 9, 8) + 23, 31);
	// center on encode
	man += 1u << (shift - 1);
	man >>= shift;
	if ((man.x >= (1 << 9)) ||
		(man.y >= (1 << 9)) ||
		(man.z >= (1 << 8)))
	{
		// overflow due to rounding, bump exponent so that color will fit
		max_exp++;
		man >>= 1;
	}
	return
		(uint(max_exp + 32) << (9 + 9 + 8)) |
		(uint(man.x   	  ) << (9 + 8    )) |
		(uint(man.y   	  ) << (8        )) |
		(uint(man.z       ));
}

float3 UnpackRGB998E6(uint bits)
{
	int e = ((bits >> (9 + 9 + 8)) & ((1 << 6) - 1)) - 32;
	int r = ((bits >> (9 + 8    )) & ((1 << 9) - 1));
	int g = ((bits >> (8        )) & ((1 << 9) - 1));
	int b = ((bits               ) & ((1 << 8) - 1));
	float3 f = {
		asfloat(uint(e + 127 - 9) << 23),
		asfloat(uint(e + 127 - 9) << 23),
		asfloat(uint(e + 127 - 8) << 23)
	};
	float3 result = { float(r), float(g), float(b) };
	return result * f;
}

uint PackRGB111110(float3 rgb)
{
	// NOTE: Could add a simple gamma2 here to improve precision of dark colors, but this does not seem necessary for now.
	// unorm 11/11/10 encoding
	uint r = uint(floor(saturate(rgb.r) * 2047.0 + 0.5));
	uint g = uint(floor(saturate(rgb.g) * 2047.0 + 0.5));
	uint b = uint(floor(saturate(rgb.b) * 1023.0 + 0.5));
	return (r << 21) | (g << 10) | b;
}

float3 UnpackRGB111110(uint rgb)
{
	float r = float((rgb >> 21)       ) * (1.0 / 2047.0);
	float g = float((rgb >> 10) & 2047) * (1.0 / 2047.0);
	float b = float((rgb      ) & 1023) * (1.0 / 1023.0);
	return float3(r, g, b);
}


//
//	float4 <-> uint2 as f16.f16.f16.f16
//

uint2 PackR16G16B16A16F(float4 In)
{
	return uint2(PackFloat2ToUInt(In.xy), PackFloat2ToUInt(In.zw));
}

float4 UnpackR16G16B16A16F(uint2 In)
{
	return float4(UnpackFloat2FromUInt(In.x), UnpackFloat2FromUInt(In.y));
}

//
//	float <-> 24bit float (8bit exp, 15 bit mantissa)
//

uint PackR24F(float In)
{
	return asuint(In) >> 8;
}

float UnpackR24F(uint In)
{
	return asfloat(In<<8);
}

//
// Common tile coord encoding
//

#define PackTileCoordXXbits(In, InMask, InBits)  ((In.x & InMask) | ((In.y & InMask) << InBits))
#define UnpackTileCoordXXbits(In,InMask, InBits) uint2(BitFieldExtractU32(In, InBits, 0), BitFieldExtractU32(In, InBits, InBits))

uint  PackTileCoord8bits(uint2 In)  { return PackTileCoordXXbits  (In, 0xFF,  8u); }
uint2 UnpackTileCoord8bits(uint In) { return UnpackTileCoordXXbits(In, 0xFF,  8u); }
uint  PackTileCoord10bits(uint2 In) { return PackTileCoordXXbits  (In, 0x3FF, 10u); }
uint2 UnpackTileCoord10bits(uint In){ return UnpackTileCoordXXbits(In, 0x3FF, 10u); }
uint  PackTileCoord12bits(uint2 In) { return PackTileCoordXXbits  (In, 0xFFF, 12u); }
uint2 UnpackTileCoord12bits(uint In){ return UnpackTileCoordXXbits(In, 0xFFF, 12u); }
uint  PackTileCoord14bits(uint2 In) { return PackTileCoordXXbits  (In, 0x3FFF, 14u); }
uint2 UnpackTileCoord14bits(uint In){ return UnpackTileCoordXXbits(In, 0x3FFF, 14u); }
uint  PackTileCoord16bits(uint2 In) { return PackTileCoordXXbits  (In, 0xFFFF,16u); }
uint2 UnpackTileCoord16bits(uint In){ return UnpackTileCoordXXbits(In, 0xFFFF,16u); }


//
// Dithering
//

// Return 0 if v is zero, 1 otherwise
float IsNonZeroFast(float v)
{
	return float(min(asuint(v) & 0x7FFFFFFF, 1u));
}

float3 IsNonZeroFast(float3 v)
{
	return float3(min(asuint(v) & 0x7FFFFFFF, 1u.xxx));
}

float DitherIfNonBlack(float Input, float Dither)
{
	// IsNonZeroFast ensures dithering is not applied when Input is zero. Input us assumed to be in [0..1].
	float NewInput = Input + Dither * IsNonZeroFast(Input);
	return saturate(NewInput);
}

float3 DitherIfNonBlack(float3 Input, float3 Dither)
{
	// IsNonZeroFast ensures dithering is not applied when Input is zero. Input us assumed to be in [0..1].
	float3 NewInput = Input + Dither * IsNonZeroFast(Input);
	return saturate(NewInput);
}

float DitherXbits(float Value, float Dither, float QuantizationScale)
{
	const float CenteredDither = Dither - 0.5f;
	const float DitheredXbits = CenteredDither * QuantizationScale;
	return DitherIfNonBlack(saturate(Value), DitheredXbits);
}

#define Dither5bits(Value, Dither) DitherXbits(Value, Dither, 1.f/31.f)
#define Dither6bits(Value, Dither) DitherXbits(Value, Dither, 1.f/63.f)
#define Dither7bits(Value, Dither) DitherXbits(Value, Dither, 1.f/127.f)
#define Dither8bits(Value, Dither) DitherXbits(Value, Dither, 1.f/255.f)