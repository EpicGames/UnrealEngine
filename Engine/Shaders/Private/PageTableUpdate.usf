// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

uint2 PageTableSize;
uint FirstUpdate;
uint NumUpdates;

Buffer<uint4> UpdateBuffer;

uint ReverseMortonCode2( uint x )
{
	x &= 0x55555555;
	x = (x ^ (x >> 1)) & 0x33333333;
	x = (x ^ (x >> 2)) & 0x0f0f0f0f;
	x = (x ^ (x >> 4)) & 0x00ff00ff;
	x = (x ^ (x >> 8)) & 0x0000ffff;
	return x;
}

void PageTableUpdateVS(
	uint VertexID : SV_VertexID,
	uint InstanceID : SV_InstanceID,
	out nointerpolation uint OutColor : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition =  float4(0, 0, 0, 1);
	OutColor = 0;

	// needs to be the same on C++ side (faster on NVIDIA and AMD)
	uint QuadsPerInstance = 8;
	uint UpdateIndex = FirstUpdate + InstanceID * QuadsPerInstance + (VertexID >> 2);

	BRANCH
	if( UpdateIndex - FirstUpdate >= NumUpdates )
	{
		return;
	}

	// UpdatePacked is R16G16B16A16_UINT
	uint4 UpdatePacked = UpdateBuffer[ UpdateIndex ];

	uint vLevel = ( UpdatePacked.w >> 0 ) & 0xff;
	uint vLogSize = ( UpdatePacked.w >> 8 ) & 0xff;
	uint vSize = 1u << vLogSize;

	// Little Endian
	uint vAddress = UpdatePacked.x | (UpdatePacked.y << 16);
	
	uint2 vPage;
	vPage.x = ReverseMortonCode2( vAddress );
	vPage.y = ReverseMortonCode2( vAddress >> 1 );

	uint2 pPage;
	pPage.x = ( UpdatePacked.z >> 0 ) & 0xff;
	pPage.y = ( UpdatePacked.z >> 8 ) & 0xff;
	
	uint2 CornerOffset = uint2(
		(VertexID >> 0) & 1,
		(VertexID >> 1) & 1
	);
	
	uint2 vCorner = vPage + vSize * CornerOffset;
	OutPosition.xy = ( (float2)vCorner / PageTableSize ) * float2( 2, -2 ) + float2( -1, 1 );

#if USE_16BIT
	// We can assume pPage fits in 6 bits and pack the final output to 16 bits
	const uint PageCoordinateBitCount = 6;
#else
	const uint PageCoordinateBitCount = 8;
#endif

	// We want to store the quantity vPagesWideInLevel (computed as VIRTUALTEXTURE_MAX_PAGETABLE_SIZE >> vLevel)
	// We then want to scale by (1/VIRTUALTEXTURE_MAX_PAGETABLE_SIZE), to avoid the need to explicitly factor this into other uniforms
	// So scale factor becomes (VIRTUALTEXTURE_MAX_PAGETABLE_SIZE >> vLevel) / VIRTUALTEXTURE_MAX_PAGETABLE_SIZE, which simplifies to (1 / exp2(vLevel)) or (1 / (1 << vLevel))
	// We will store vLevel in the page table and calculate (1 / (1 << vLevel)) on load
	// Also nice that this avoids need to depend on specific value of VIRTUALTEXTURE_MAX_PAGETABLE_SIZE
	uint Page = vLevel;
	Page |= pPage.x << 4;
	Page |= pPage.y << (4 + PageCoordinateBitCount);
	
	OutColor = Page;
}

void PageTableUpdatePS_1(
	nointerpolation uint InColor : TEXCOORD0,
	out uint OutColor : SV_Target0
	)
{
	OutColor = InColor; 
}

void PageTableUpdatePS_2(
	nointerpolation uint InColor : TEXCOORD0,
	out uint2 OutColor : SV_Target0
	)
{
	// splat output to RGBA channels, color write mask will be configured to write to the correct channel
	OutColor = InColor.xx;
}

void PageTableUpdatePS_4(
	nointerpolation uint InColor : TEXCOORD0,
	out uint4 OutColor : SV_Target0
	)
{
	// splat output to RGBA channels, color write mask will be configured to write to the correct channel
	OutColor = InColor.xxxx;
}
