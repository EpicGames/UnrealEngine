// Copyright Epic Games, Inc. All Rights Reserved.

//------------------------------------------------------- INCLUDES

#include "Common.ush"
#include "ScreenPass.ush"
#include "PaniniProjection.ush"


//------------------------------------------------------- PARAMETERS

float2 ScreenSpaceToPaniniFactor;
float2 PaniniToScreenSpaceFactor;
FScreenTransform DispatchThreadIdToDestViewportUV;

// only for MainVS()
// .x:HalfFOV .y:tan(HalfFov) .z:CylinderDistortion(0=none, 1:full) .w:Correction to scale Y the same as X is scaled in the center
float PaniniD;
float PaniniS;
float ScreenPosScale;

RWTexture2D<float2> DistortingDisplacementOutput;
RWTexture2D<float2> UndistortingDisplacementOutput;


//------------------------------------------------------- FUNCTION

float2 PaniniInverseProjection(float2 ON, float d, float s)
{
	// line D equation through DN: A x + B z + C = 0
	float A = 1.0 + d;
	float B = - ON.x;
	float C = ON.x * d;
	
	// find intersection K(x,z) between line DN and unit circle with center O
	// solve: x^2 + z^2 = 1, z < 0, A x + B z + C = 0
	// ends up with polynom: a z^2 + b z + x = 0
	float a = 1.0 + (B * B) / (A * A);
	float b = 2.0 * (B * C) / (A * A);
	float c = (C * C) / (A * A) - 1.0;

	float z = (-b - sqrt(b * b - 4.0 * a * c)) / (2.0 * a);

	float CosPhi = -z;
	float SinPhi = sqrt(1.0 - CosPhi * CosPhi) * sign(ON.x);

	float S = (d + 1.0) / (d + CosPhi);

	float OMx = SinPhi / CosPhi;
	float PaniniDirectionXZInvLength = rsqrt(1.0 + OMx * OMx);

	float TanTheta = ON.y / (S * lerp(1.0, 1.0 / CosPhi, s));
	float OMy = TanTheta / PaniniDirectionXZInvLength;

	return float2(OMx, OMy);
}

// Panini projection in the screen pos
// @param InScreenPos: the position to project in the screen space
// @return the panini projected postion in screen space
float2 PaniniProjectionScreenPos(float2 InScreenPos)
{
	const float2 PaniniDirection = InScreenPos * ScreenSpaceToPaniniFactor;
	const float2 PaniniPosition = PaniniProjection(PaniniDirection, PaniniD, PaniniS);

	// Return the new position back in the screen space frame
	return PaniniPosition * PaniniToScreenSpaceFactor * ScreenPosScale;
}

float2 PaniniInverseProjectionScreenPos(float2 InScreenPos)
{
	float2 PaniniPosition = InScreenPos * ScreenSpaceToPaniniFactor * rcp(ScreenPosScale);
	float2 PaniniDirection = PaniniInverseProjection(PaniniPosition, PaniniD, PaniniS);
	
	// Works arround floating point precision problem on the computation of OMx
	const float PrecisionThreshold = 0.003;
	BRANCH
	if (abs(PaniniPosition.x) < PrecisionThreshold)
	{
		float Lerp = saturate(abs(PaniniPosition.x) / PrecisionThreshold);
		PaniniDirection.x = Lerp * PaniniInverseProjection(float2(PrecisionThreshold * sign(PaniniPosition.x), PaniniPosition.y), PaniniD, PaniniS).x;
	}

	// Return the new position back in the screen space frame
	return PaniniDirection * PaniniToScreenSpaceFactor;
}


//------------------------------------------------------- ENTRY POINT

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	float2 DestViewportUV = ApplyScreenTransform(float2(DispatchThreadId.xy), DispatchThreadIdToDestViewportUV);
	float2 DestScreenPos = ViewportUVToScreenPos(DestViewportUV);
	
	BRANCH
	if (DispatchThreadId.z == 0)
	{
		float2 SrcScreenPos = PaniniProjectionScreenPos(DestScreenPos);
		float2 SrcViewportUV = ScreenPosToViewportUV(SrcScreenPos);
		
		DistortingDisplacementOutput[DispatchThreadId.xy] = SrcViewportUV - DestViewportUV;
	}
	else
	{
		float2 SrcInverseScreenPos = PaniniInverseProjectionScreenPos(DestScreenPos);
		float2 SrcInverseViewportUV = ScreenPosToViewportUV(SrcInverseScreenPos);
		
		UndistortingDisplacementOutput[DispatchThreadId.xy] = SrcInverseViewportUV - DestViewportUV;
	}
}
