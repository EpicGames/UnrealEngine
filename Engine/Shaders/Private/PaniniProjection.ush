// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PaniniProjection.usf: Panini implementation for the node material function
	and upsample post processing pass
=============================================================================*/
  
#pragma once

/*
* Reproject a ray direction into the panini view's plan.
*	Source: http://tksharpless.net/vedutismo/Pannini/panini.pdf
*
* Panini frame:
*                                        |
*                                        |   /
*                                      M | /
*                                        +
*                                      / |
*                        ->          /   |
*                        z         /     |                   (view direction)
*        - + - - - - - - < - - - + - - - + - - + - - - - - - - - >
*          D                     O       |A
*                                        |
*                                        |
*                                (panini view plan)
*
*		O: origin of the frame
*		A: origin of the panini view plane (0.0, 0.0, -1.0)
*		D: panini projection center (0.0, 0.0, d)
*		OM: the direction of the ray.
*
* @param OM - The direction of the ray in the panini frame. The z component is implicitely -1.0f in the panini frame.
* @param d - The panini projection center's distance from the panini frame's origin.
*		0.0f: Panini projection doesn't change anything, it will return PaniniDirection.
*		1.0f: Panini
*		@assert d>=0
* @param s - The panini projection hard vertical compression fade (lerp). Works with any number in the real domain.
*		0.0f: No vertical compression
*		1.0f: Hard vertical compression
* @return - Returns the position (x, y) of the panini projection in the panini view plane at z=-1.0f in the panini frame
*/
float2 PaniniProjection(float2 OM, float d, float s)
{
	float PaniniDirectionXZInvLength = rsqrt(1.0f + OM.x * OM.x);
	float SinPhi = OM.x * PaniniDirectionXZInvLength;
	float TanTheta = OM.y * PaniniDirectionXZInvLength;
	float CosPhi = sqrt(1.0f - SinPhi * SinPhi);
	float S = (d + 1.0f) / (d + CosPhi);

	return S * float2(SinPhi, lerp(TanTheta, TanTheta / CosPhi, s));
}
