// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticipatingMediaCommon.ush
=============================================================================*/

#pragma once

#include "Common.ush"
#include "FastMath.ush"

#define PARTICIPATING_MEDIA_MIN_MFP_METER		0.000000000001f
#define PARTICIPATING_MEDIA_MIN_EXTINCTION		0.000000000001f
#define PARTICIPATING_MEDIA_MIN_TRANSMITTANCE	0.000000000001f

// This can be used to hide color banding due to precision when sampling the low resolution SIMPLE_VOLUME LUT used with low diffuse color and boosted diffuse Lumen GI.
// Make sure SubstrateBSDFContext.PixelCoord is correct for such use case. This define allow to selectively only pay that cost when it is really needed, e.g. ray traced translucent with LumenGI hit lighting.
#ifndef SUBSTREATE_SIMPLEVOLUME_ENVALBEDO_LUT_JITTERING
#define SUBSTREATE_SIMPLEVOLUME_ENVALBEDO_LUT_JITTERING 0
#endif

//---------------------------------------------
// Participating media representation
//---------------------------------------------

struct FParticipatingMedia
{
	float3 ScatteringCoef;	// sigma_s (1/meter)
	float3 AbsorptionCoef;	// sigma_a (1/meter)
	float3 ExtinctionCoef;	// sigma_t (1/meter)
	float3 MeanFreePath;	// (meter)
	float3 Albedo;			// Represent sigma_s / sigma_t (unitless)
	float3 BaseColor;		// Represent the reflectance albedo resulting from single+multiple scattering assuming an isotropic phase function (unitless)
};

// GetBaseColorFromAlbedo: returns the color of the participating media resulting from the single+multiple subsurface scattering. 
// GetAlbedoFromBaseColor is the inverse transform.
// [Kulla and Conty 2017, "Revisiting Physically Based Shading at Imageworks"] https://blog.selfshadow.com/publications/s2017-shading-course/imageworks/s2017_pbs_imageworks_slides_v2.pdf, slide 62 
// [d'Eon, "A Hitchhiker's guide to multiple scattering"] or http://www.eugenedeon.com/wp-content/uploads/2016/09/hitchhikers_v0.1.3.pdf
float3 GetBaseColorFromAlbedo(const float3 Albedo, const float g = 0.0f)
{
	const float3 s = sqrt((1 - Albedo) / (1.0f - Albedo * g));
	const float3 BaseColor = ((1.0f - s) * (1 - 0.139 * s)) / (1.0f + 1.17 * s);
	return BaseColor;
}
float3 GetAlbedoFromBaseColor(const float3 BaseColor, const float g = 0.0f)
{
	const float3 s = 4.09712 + 4.20863 * BaseColor - sqrt(9.59217 + 41.6808 * BaseColor + 17.7126 * BaseColor * BaseColor);
	const float3 Albedo = (1.0f - s * s) / (1.0f - g * s * s);
	return Albedo;
}

// The mean free path must be in meters
FParticipatingMedia CreateMediumFromAlbedoMFP(float3 Albedo, float3 MeanFreePathMeters)
{
	FParticipatingMedia PM = (FParticipatingMedia)0;
	PM.Albedo = Albedo;
	PM.BaseColor = GetBaseColorFromAlbedo(Albedo);
	PM.MeanFreePath = MeanFreePathMeters;
	PM.ExtinctionCoef = 1.0f / max(PARTICIPATING_MEDIA_MIN_MFP_METER, PM.MeanFreePath);
	PM.ScatteringCoef = PM.Albedo * PM.ExtinctionCoef;
	PM.AbsorptionCoef = max(0.0f, PM.ExtinctionCoef - PM.ScatteringCoef);
	return PM;
}

// The mean free path must be in meters
FParticipatingMedia CreateMediumFromBaseColorMFP(float3 BaseColor, float3 MeanFreePathMeters)
{
	FParticipatingMedia PM = (FParticipatingMedia)0;
	PM.Albedo = GetAlbedoFromBaseColor(BaseColor);
	PM.BaseColor = BaseColor;
	PM.MeanFreePath = MeanFreePathMeters;
	PM.ExtinctionCoef = 1.0f / max(PARTICIPATING_MEDIA_MIN_MFP_METER, PM.MeanFreePath);
	PM.ScatteringCoef = PM.Albedo * PM.ExtinctionCoef;
	PM.AbsorptionCoef = max(0.0f, PM.ExtinctionCoef - PM.ScatteringCoef);
	return PM;
}



//---------------------------------------------
// Phase functions
//---------------------------------------------

float IsotropicPhase()
{
	return 1.0f / (4.0f * PI);
}

// Follows PBRT convention http://www.pbr-book.org/3ed-2018/Volume_Scattering/Phase_Functions.html#PhaseHG
float HenyeyGreensteinPhase(float G, float CosTheta)
{
	// Reference implementation (i.e. not schlick approximation). 
	// See http://www.pbr-book.org/3ed-2018/Volume_Scattering/Phase_Functions.html
	float Numer = 1.0f - G * G;
	float Denom = 1.0f + G * G + 2.0f * G * CosTheta;
	return Numer / (4.0f * PI * Denom * sqrt(Denom));
}

float RayleighPhase(float CosTheta)
{
	float Factor = 3.0f / (16.0f * PI);
	return Factor * (1.0f + CosTheta * CosTheta);
}

// Schlick phase function approximating henyey-greenstein
float SchlickPhaseFromK(float K, float CosTheta)
{
	const float SchlickPhaseFactor = 1.0f + K * CosTheta;
	const float PhaseValue = (1.0f - K * K) / (4.0f * PI * SchlickPhaseFactor * SchlickPhaseFactor);
	return PhaseValue;
}
float SchlickPhase(float G, float CosTheta)
{
	const float K = 1.55f * G - 0.55f * G * G * G;
	return SchlickPhaseFromK(K, CosTheta);
}

// Follows PBRT convention http://www.pbr-book.org/3ed-2018/Light_Transport_II_Volume_Rendering/Sampling_Volume_Scattering.html#SamplingPhaseFunctions
float HenyeyGreensteinPhaseInvertCDF(float E, float G)
{
	// The naive expression requires a special case for G==0, rearranging terms a bit
	// gives the following result which does not require any special cases
	float t0 = (1.0 - G) + G * E;
	float t1 = (1.0 - E) + E * E;
	float t2 = t1 + (G * E) * t0;
	float t3 = (2.0 * E - 1.0) - G * G;
	float Num = t3 + (2.0 * G) * t2;
	float Den = t0 + G * E;
	return Num / (Den * Den);
}

// Follows PBRT convention http://www.pbr-book.org/3ed-2018/Light_Transport_II_Volume_Rendering/Sampling_Volume_Scattering.html#SamplingPhaseFunctions
float4 ImportanceSampleHenyeyGreensteinPhase(float2 E, float G)
{
	float Phi = 2.0f * PI * E.x;
	float CosTheta = HenyeyGreensteinPhaseInvertCDF(E.y, G);
	float SinTheta = sqrt(max(0.0f, 1.0f - CosTheta * CosTheta));

	float3 H = float3(SinTheta * sin(Phi), SinTheta * cos(Phi), CosTheta);

	return float4(H, HenyeyGreensteinPhase(G, CosTheta));
}

// Returns CosTheta given a random value in [0,1)
float RayleighPhaseInvertCdf(float E)
{
	// [Frisvad, 2011] "Importance sampling the Rayleigh phase function"
	// Optical Society of America. Journal A: Optics, Image Science, and Vision
	float Z = E * 4.0 - 2.0;
	float InvZ = sqrt(Z * Z + 1.0);
	float u = pow(Z + InvZ, 1.0 / 3.0); // cbrt
	return u - rcp(u);
}

float4 ImportanceSampleRayleigh(float2 E)
{
	float Phi = 2.0f * PI * E.x;
	float CosTheta = RayleighPhaseInvertCdf(E.y);
	float SinTheta = sqrt(max(0.0f, 1.0f - CosTheta * CosTheta));
	float3 H = float3(SinTheta * sin(Phi), SinTheta * cos(Phi), CosTheta);
	return float4(H, RayleighPhase(CosTheta));
}


//---------------------------------------------
// Utilities
//---------------------------------------------

float3 TransmittanceToExtinction(in float3 TransmittanceColor, in float ThicknessMeters)
{
	// TransmittanceColor	= exp(-Extinction * Thickness)
	// Extinction			= -log(TransmittanceColor) / Thickness
	return -log(clamp(TransmittanceColor, PARTICIPATING_MEDIA_MIN_TRANSMITTANCE, 1.0f)) / max(PARTICIPATING_MEDIA_MIN_MFP_METER, ThicknessMeters);
}

float3 TransmittanceToMeanFreePath(in float3 TransmittanceColor, in float ThicknessMeters)
{
	return 1.0f / max(PARTICIPATING_MEDIA_MIN_EXTINCTION, TransmittanceToExtinction(TransmittanceColor, ThicknessMeters));
}

float3 ExtinctionToTransmittance(in float3 Extinction, in float ThicknessMeters)
{
	return exp(-Extinction * ThicknessMeters);
}

float3 MeanFreePathToTransmittance(in float3 MeanFreePath, in float ThicknessMeters)
{
	return exp(-ThicknessMeters / max(MeanFreePath, PARTICIPATING_MEDIA_MIN_MFP_METER));
}


//---------------------------------------------
// Lighting evaluation function for an Isotropic participating media a slab of 1 meter
//---------------------------------------------

float3 IsotropicMediumSlabDirectionalAlbedoFade(float3 BaseColor, float3 MFP)
{
	float3 Fade;
	// Ensure the scattering fade out to 0 as the BaseColor approach 0. This Starts when BaseColor is below 10%.
	const float BaseColorFadesOutBelowPercentage = 10.0f;
	Fade = saturate(BaseColor * BaseColorFadesOutBelowPercentage);
	// And also fade out BaseColor to zero from MFP = 20meters (the last measured MFP for which we have fit the curve) to a large MFP=1000 meters
	const float FitLastMeasuredSampleMFP = 20.0f;
	const float AlbedoIsZeroForMFP = 1000.0f;
	Fade*= saturate(1.0f - (MFP - FitLastMeasuredSampleMFP) / (AlbedoIsZeroForMFP - FitLastMeasuredSampleMFP));

	return Fade;
}

// The ALU version is a lot simpler as compared to the LUT version: it is not view and not light dependent.
float3 IsotropicMediumSlabPunctualDirectionalAlbedoALU(FParticipatingMedia PM)
{
	// Our measurement are only valid for a MFP of 0.01 meter so we clamp input MFP to that.
	// This is relatively ok since all computation are done for a slab of 1 meter.
	const float3 MFP = max(0.01f, PM.MeanFreePath);

	const float3 EvaluateForBaseColor1	= 0.0855674 / (0.237742 + (MFP + ((0.0310849 - MFP) / (1.95492 * MFP + 2.07238))));
	const float3 EvaluateForBaseColor01 = 0.0167964 / (0.541037 * (pow(1.17902, (-4.33046) / MFP) * (-0.294969 + MFP)) + 0.797592);

	// Lerp between the two extreme BaseColor curves. Measurement range was [0.1f, 1.0f].
	float3 FinalEvaluate = lerp(EvaluateForBaseColor01, EvaluateForBaseColor1, (PM.BaseColor - 0.1f) / (1.0f - 0.1f));
	return FinalEvaluate * IsotropicMediumSlabDirectionalAlbedoFade(PM.BaseColor, MFP);
}

// Platforms that do not support shared sampler will in fact quickly reach the limit of samplers in material pixel shaders.
// This is mostly true for OpenGLES. In this case, we enforce disabling SimpleVolume LUT and use the lower quality ALU based one.
// SUBSTRATE_TODO Implement an ALU based solution that give the same view dependent high quality as the LUT.
#define USE_LUT_SIMPLEVOLUME (SUPPORTS_INDEPENDENT_SAMPLERS)

float IsotropicMediumSlabDirectionalAlbedoFadeLUT(float MFP)
{
	// Fade out directional albedo to zero from MFP = 10meters (the last measured MFP for which we have fit the curve) to a large MFP=1000 meters
	const float FitLastMeasuredSampleMFP = 10.0f;
	const float AlbedoIsZeroForMFP = 640.0f;	// This limit is realted to the maximum value MFP that can be stored in the material buffer.
	const float Fade = saturate(1.0f - (MFP - FitLastMeasuredSampleMFP) / (AlbedoIsZeroForMFP - FitLastMeasuredSampleMFP));
	return Fade;
}

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedSimpleVolumeTextureSampler	View.SharedBilinearClampedSampler
	#define SharedSimpleVolumeEnvTextureSampler	View.SharedBilinearClampedSampler
#else
	#define SharedSimpleVolumeTextureSampler	View.SimpleVolumeTextureSampler
	#define SharedSimpleVolumeEnvTextureSampler	View.SimpleVolumeEnvTextureSampler
#endif

float IsotropicMediumSlabPunctualDirectionalAlbedoLUT(float MFP, float BaseColor, float NoV, float NoL)
{
	// LUT content
	//		View angle	from 0 to 7/8	with steps of 1/8 => 8  texels
	//		Light angle	from 0 to 1  	with steps of 1/8 => 9  texels
	//		BaseColor	from 0 to 1		with steps of 0.1 => 11 texels
	//		MFP			from 0 to 10	with steps of 1   => 11 texels
	// 
	// Indexing  LUTINDEX(V, L, B, M) (V * DimL * DimBC * DimMFP + L * DimBC * DimMFP  + B * DimMFP  + M)

	// LUT dimension for input parameters: MFP, BaseColor, NoL, NoV
	const float4 LUTDim = float4(11.0f, 11.0, 9.0f, 8.0f);
	const float4 LUTDimInv = 1.0f / LUTDim;
	const float4 LUTMin = 0.5f * LUTDimInv;
	const float4 LUTMax = (LUTDim - 0.5f) * LUTDimInv;
	const float4 LUTLen = (LUTDim - 1.0f) * LUTDimInv;

	const float PIOver2 = PI / 2.0f;
	const float LightAngleNorm = saturate(acosFast(saturate(NoL)) / PIOver2);
	const float ViewAngleNorm  = saturate(acosFast(saturate(NoV)) / PIOver2);

	float4 UVs = float4(MFP / 10.0f, BaseColor, LightAngleNorm, ViewAngleNorm);	// Input parameters in [0,1]
	UVs = LUTMin + LUTLen * UVs;												// Correct in order to ignore texel border

	float FourthCoord = ViewAngleNorm * LUTDim.w;
	float FourthCoord0 = floor(FourthCoord);
	      FourthCoord0 = min(FourthCoord0, LUTDim.w - 1.0);
	float FourthCoord1 = FourthCoord0 + 1.0;
	      FourthCoord1 = min(FourthCoord1, LUTDim.w - 1.0);
	float FourthCoordLerp = saturate(FourthCoord - FourthCoord0);

	const float LUT3DDimZ    = LUTDim.z * LUTDim.w;
	const float LUT4DUvDimZ  = (LUTDim.z - 1.0) / LUT3DDimZ;	// Size of third dimension along Z
	const float LUT4DUvStepW = (LUTDim.z      ) / LUT3DDimZ; // Size of step to sample the 4th dimensions along Z
	const float EdgeOffset = (0.5) / LUT3DDimZ;

	float3 UVW0 = float3(UVs.x, UVs.y, EdgeOffset + FourthCoord0 * LUT4DUvStepW + LightAngleNorm * LUT4DUvDimZ);
	float3 UVW1 = float3(UVs.x, UVs.y, EdgeOffset + FourthCoord1 * LUT4DUvStepW + LightAngleNorm * LUT4DUvDimZ);

	float Value0 = View.SimpleVolumeTexture.SampleLevel(SharedSimpleVolumeTextureSampler, UVW0, 0);
	float Value1 = View.SimpleVolumeTexture.SampleLevel(SharedSimpleVolumeTextureSampler, UVW1, 0);

	float DirectionalAlbedo = lerp(Value0, Value1, FourthCoordLerp);

	const float Fade = IsotropicMediumSlabDirectionalAlbedoFadeLUT(MFP);
	return DirectionalAlbedo * Fade;
}

float3 IsotropicMediumSlabPunctualDirectionalAlbedoLUT(FParticipatingMedia PM, float NoV, float NoL)
{
	return float3(
		IsotropicMediumSlabPunctualDirectionalAlbedoLUT(PM.MeanFreePath.r, PM.BaseColor.r, NoV, NoL),
		IsotropicMediumSlabPunctualDirectionalAlbedoLUT(PM.MeanFreePath.g, PM.BaseColor.g, NoV, NoL),
		IsotropicMediumSlabPunctualDirectionalAlbedoLUT(PM.MeanFreePath.b, PM.BaseColor.b, NoV, NoL)
	);
}

float3 IsotropicMediumSlabPunctualDirectionalAlbedo(FParticipatingMedia PM, float NoV, float NoL)
{
#if USE_LUT_SIMPLEVOLUME
	const float3 SlabDirectionalAlbedo = IsotropicMediumSlabPunctualDirectionalAlbedoLUT(PM, NoV, NoL);
#else
	const float3 SlabDirectionalAlbedo = IsotropicMediumSlabPunctualDirectionalAlbedoALU(PM);
#endif
	return SlabDirectionalAlbedo;
}



// The ALU version is a lot simpler as compared to the LUT version: it is not view dependent.
float3 IsotropicMediumSlabEnvDirectionalAlbedoALU(FParticipatingMedia PM)
{
	// Our measurement are only valid for a MFP of 0.01 meter so we clamp input MFP to that.
	// This is relatively ok since all computation are done for a slab of 1 meter.
	const float3 MFP = max(0.01f, PM.MeanFreePath);

	const float3 EvaluateForBaseColor1	= 0.00231881 + (0.51379 / (pow(MFP, 1.03577) + 0.510465)); 
	const float3 EvaluateForBaseColor01 = 0.189167 / (1.55597 + (MFP + pow(0.182843, 0.0666775 + MFP)));

	// Lerp between the two extreme BaseColor curves. Measurement range was [0.1f, 1.0f].
	float3 FinalEvaluate = lerp(EvaluateForBaseColor01, EvaluateForBaseColor1, (PM.BaseColor - 0.1f) / (1.0f - 0.1f));
	return FinalEvaluate * IsotropicMediumSlabDirectionalAlbedoFade(PM.BaseColor, MFP);
}

float3 IsotropicMediumSlabTransmittance(FParticipatingMedia PM, float SlabThickness, float NoV)
{
	const float3 SafeExtinctionThreshold = 0.000001f;
	const float3 SafeExtinctionCoefficients = max(SafeExtinctionThreshold, PM.ExtinctionCoef);

	const float PathLength = SlabThickness / max(0.0001f, abs(NoV));
	const float3 SafePathSegmentTransmittance = exp(-SafeExtinctionCoefficients * PathLength);

	return SafePathSegmentTransmittance;
}

float IsotropicMediumSlabEnvDirectionalAlbedoLUT(float MFP, float BaseColor, float NoV, float Jittering)
{
	// LUT content
	//		View angle	from 0 to 7/8	with steps of 1/8 => 8  texels
	//		Light angle	none
	//		BaseColor	from 0 to 1		with steps of 0.1 => 11 texels
	//		MFP			from 0 to 10	with steps of 1   => 11 texels
	// 
	// Indexing  LUTINDEX(V, L, B, M) (V * DimL * DimBC * DimMFP + L * DimBC * DimMFP  + B * DimMFP  + M)

	// LUT dimension for input parameters: MFP, BaseColor, NoL, NoV
	const float3 LUTDim = float3(11.0f, 11.0, 8.0f);
	const float3 LUTDimInv = 1.0f / LUTDim;
	const float3 LUTMin = 0.5f * LUTDimInv;
	const float3 LUTMax = (LUTDim - 0.5f) * LUTDimInv;
	const float3 LUTLen = (LUTDim - 1.0f) * LUTDimInv;

	const float PIOver2 = PI / 2.0f;
	const float ViewAngleNorm = saturate(acosFast(saturate(NoV)) / PIOver2);

	float3 UVs = float3(MFP / 10.0f, BaseColor, ViewAngleNorm);	// Input parameters in [0,1]

	UVs.z += Jittering * LUTDimInv.z;							// Add some jittering to break color resulting from precision issue when sampling the low resolution LUT (particularly visible in shadow, when GI diffuse color is boosted along NoV)

	UVs = LUTMin + LUTLen * UVs;								// Correct in order to ignore texel border

	float EnvDirectionalAlbedo = View.SimpleVolumeEnvTexture.SampleLevel(SharedSimpleVolumeEnvTextureSampler, UVs, 0);

	const float Fade = IsotropicMediumSlabDirectionalAlbedoFadeLUT(MFP);
	return EnvDirectionalAlbedo * Fade;
}

float3 IsotropicMediumSlabEnvDirectionalAlbedoLUT(FParticipatingMedia PM, float NoV, uint2 PixelCoord)
{
#if SUBSTREATE_SIMPLEVOLUME_ENVALBEDO_LUT_JITTERING
	const float Jittering = 1.0 * (InterleavedGradientNoise(PixelCoord, float(ResolvedView.StateFrameIndexMod8)) - 0.5);
#else
	const float Jittering = 0.0f;
#endif

	return float3(
		IsotropicMediumSlabEnvDirectionalAlbedoLUT(PM.MeanFreePath.r, PM.BaseColor.r, NoV, Jittering),
		IsotropicMediumSlabEnvDirectionalAlbedoLUT(PM.MeanFreePath.g, PM.BaseColor.g, NoV, Jittering),
		IsotropicMediumSlabEnvDirectionalAlbedoLUT(PM.MeanFreePath.b, PM.BaseColor.b, NoV, Jittering)
	);
}

float3 IsotropicMediumSlabEnvDirectionalAlbedo(FParticipatingMedia PM, float NoV, uint2 PixelCoord)
{
#if USE_LUT_SIMPLEVOLUME
	float3 SlabEnvAlbedo = IsotropicMediumSlabEnvDirectionalAlbedoLUT(PM, NoV, PixelCoord);
#else
	// The environment response of has been measured for an hemisphere having a uniform radiance of 1. 
	// This DirectionalAlbedo also contains the integral over the hemisphere according to the phase function.
	// The spherical harmonic used for environment lighting contains the diffuse integral over the hemisphere as DiffLambert = int_{\omega} {1 * NoL d\Omega} = PI.
	// We do not want to be affected by the NoL term here since The DirectionalAlbedo also contains the integral over the hemisphere according to the phase function.
	// Integral of a value=1 over the Hemisphere is int_{\omega} {1  d\Omega} = 2PI.
	// So, as an approximation, we recover the uniform hemisphere luminance as UniformEnvL = DiffLambert * (1 / PI) * (2 * PI).
	// We consider that the diffuse integration of the environment SH over the hemisphere multiplied with the light response will be a good approximation similar to the "split sum integral".
	float3 SlabEnvAlbedo = IsotropicMediumSlabEnvDirectionalAlbedoALU(PM) * ((1 / PI) * (2 * PI));
#endif
	return SlabEnvAlbedo;
}
