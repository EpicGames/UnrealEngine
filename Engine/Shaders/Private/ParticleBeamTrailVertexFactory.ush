// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleBeamTrailVertexFactory.hlsl: Particle vertex factory shader code.
=============================================================================*/

#include "VertexFactoryCommon.ush"

struct FVertexFactoryInput
{
	float4	Position				: ATTRIBUTE0;
	float3	OldPosition				: ATTRIBUTE1;
	float4	SizeRotSubImage			: ATTRIBUTE2;
	float4	TexCoord				: ATTRIBUTE3;
	float4	Color					: ATTRIBUTE4;
#if USE_DYNAMIC_PARAMETERS
	float4 DynamicParameter0		: ATTRIBUTE5;
#endif	//USE_DYNAMIC_PARAMETERS

	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()  
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

struct FVertexFactoryInterpolantsVSToPS
{
	TANGENTTOWORLD_INTERPOLATOR_BLOCK

#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2] : PACKED_TEXCOORDS;
#endif

#if NEEDS_PARTICLE_COLOR
	float4  Color				: TEXCOORD3;
#endif

#if USE_DYNAMIC_PARAMETERS
	float4 DynamicParameter0	: TEXCOORD4;
#endif	//USE_DYNAMIC_PARAMETERS
};

struct FVertexFactoryIntermediates
{
	/** The color of the sprite. */
	float4 Color;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int UVIndex=0; UVIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++UVIndex)
	{
		Result.TexCoords[(UVIndex * 2) + 0] = Interpolants.TexCoords[UVIndex].xy;
		Result.TexCoords[(UVIndex * 2) + 1] = Interpolants.TexCoords[UVIndex].zw;
	}
	#if (NUM_TEX_COORD_INTERPOLATORS & 1) == 1
		Result.TexCoords[NUM_TEX_COORD_INTERPOLATORS - 1] = Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy;
	#endif
#endif // NUM_TEX_COORD_INTERPOLATORS

	Result.VertexColor = 1;

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.Color;
#endif

	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.TangentToWorld = AssembleTangentToWorld( TangentToWorld0, TangentToWorld2 );
	Result.UnMirrored = 1;

#if LIGHTMAP_UV_ACCESS
	#if NUM_TEX_COORD_INTERPOLATORS > 0
		Result.LightmapUVs = Interpolants.TexCoords[0].xy;
	#else
		Result.LightmapUVs = 0;
	#endif
#endif

	Result.TwoSidedSign = 1;

#if USE_DYNAMIC_PARAMETERS
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter0;
#endif	//USE_DYNAMIC_PARAMETERS

	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Input.Color;
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3());
	Result.Particle.Color = Intermediates.Color;
	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

	// Previous frame not handled deliberately. Lacks necessary information and
	// primitives using this VF are usually transparent and hence don't output velocity
	Result.PrevFrameLocalToWorld = GetPrimitiveDataFromUniformBuffer().LocalToWorld;

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	for(int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = Input.TexCoord.xy;
	}
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float3 SafeNormalize(float3 V)
{
	return V / sqrt(max(dot(V,V),0.01));
}

float4 SimulationToTranslatedWorld(float3 SimPosition)
{
	if ( BeamTrailVF.bUseLocalSpace )
	{
		FDFMatrix Transform = GetPrimitiveDataFromUniformBuffer().LocalToWorld;
		return DFTransformLocalToTranslatedWorld(SimPosition, Transform, ResolvedView.PreViewTranslation);
	}
	else
	{
		FLWCVector3 WorldPosition = MakeLWCVector3(BeamTrailVF.LWCTile, SimPosition);
		FLWCVector3 TranslatedWorldPosition = LWCAdd(WorldPosition, ResolvedView.TileOffset.PreViewTranslation);
		return float4(LWCToFloat(TranslatedWorldPosition), 1.0f);
	}
}

void GetTangents(FVertexFactoryInput Input,out float4 Right,out float4 Up)
{
	float4	Position			= SimulationToTranslatedWorld(Input.Position.xyz),
			OldPosition			= SimulationToTranslatedWorld(Input.OldPosition.xyz);

	float3	CameraDirection		= -GetCameraVectorFromTranslatedWorldPosition(ResolvedView, Position.xyz),
			ParticleDirection	= SafeNormalize(Position.xyz - OldPosition.xyz);

	float4	Right_Square		= BeamTrailVF.CameraRight,
			Up_Square			= BeamTrailVF.CameraUp;			
			
	float4	Right_Rotated		= (-1.0 * cos(Input.SizeRotSubImage.z) * Up_Square) + (sin(Input.SizeRotSubImage.z) * Right_Square),
			Up_Rotated			= (       sin(Input.SizeRotSubImage.z) * Up_Square) + (cos(Input.SizeRotSubImage.z) * Right_Square);

	float4	Right_Velocity		= float4( SafeNormalize( cross( CameraDirection, ParticleDirection	) ), 0.0 ),
			Up_Velocity			= float4( ParticleDirection, 0.0 );

	//	enum EParticleScreenAlignment
	//	{
	//		PSA_Square,
	//		PSA_Rectangle,
	//		PSA_Velocity
	//	};
	Right				= BeamTrailVF.ScreenAlignment.x > 1.5f ? Right_Velocity : Right_Rotated;
	Up					= BeamTrailVF.ScreenAlignment.x > 1.5f ? Up_Velocity	: Up_Rotated;
}

float4 CalcWorldPosition(FVertexFactoryInput Input)
{
	return SimulationToTranslatedWorld(Input.Position.xyz);
}

/** derive basis vectors */
float3x3 CalcTangentBasis(FVertexFactoryInput Input)
{
	float4	Right,
			Up;
	GetTangents(Input,Right,Up);
	return float3x3(
			Right.xyz,
			Up.xyz,
			-normalize(cross(Right.xyz,Up.xyz))
			);
}

FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.Color = Input.Color;
	return Intermediates;
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcWorldPosition(Input);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcTangentBasis(Input)[2];
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 InWorldPosition)
{
	return InWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int UVIndex=0; UVIndex < NUM_TEX_COORD_INTERPOLATORS / 2; ++UVIndex)
	{
		Interpolants.TexCoords[UVIndex].xy = CustomizedUVs[UVIndex * 2 + 0];
		Interpolants.TexCoords[UVIndex].zw = CustomizedUVs[UVIndex * 2 + 1];
	}
	#if (NUM_TEX_COORD_INTERPOLATORS & 1) == 1
		Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].xy = CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS - 1];
		Interpolants.TexCoords[NUM_TEX_COORD_INTERPOLATORS / 2].zw = 0.0f;
	#endif
#endif	// NUM_MATERIAL_TEXCOORDS

#if NEEDS_PARTICLE_COLOR
	Interpolants.Color		= Intermediates.Color;
#endif

#if USE_DYNAMIC_PARAMETERS
	Interpolants.DynamicParameter0 = Input.DynamicParameter0;
#endif	//USE_DYNAMIC_PARAMETERS

	float3x3 TangentToLocal = CalcTangentBasis(Input);
	float3x3 TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3());

	Interpolants.TangentToWorld0.xyz = TangentToWorld[0];
	Interpolants.TangentToWorld0.w = 0;
	Interpolants.TangentToWorld2 = float4(TangentToWorld[2], sign(determinant(TangentToLocal)));

	return Interpolants;
}

float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetWorldPosition(Input, Intermediates);
}

// local position relative to instance
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return CalcTangentBasis(Input);
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#include "VertexFactoryDefaultInterface.ush"