// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleBoundsShader.usf: Shader to compute the bounds of GPU particles.
==============================================================================*/

/*------------------------------------------------------------------------------
	Compile time parameters:

		THREAD_COUNT - The number of threads to launch per workgroup.
------------------------------------------------------------------------------*/

#include "Common.ush"
#include "ParticleSimulationCommon.ush"

/** The number of threads with which to perform raking. */
#define RAKING_THREADS 32
/** Padding required for raking operations. */
#define RAKING_PADDING (RAKING_THREADS >> 1)
/** Maximum floating point value. */
#define MAX_FLT (+3.402823466e+38f)
/** Minimum floating point value. */
#define MIN_FLT (-3.402823466e+38f)

/** Input buffer containing particle indices. */
Buffer<float4> InParticleIndices;
float2 TilePageScale;

/** Texture containing particle positions. */
Texture2D PositionTexture;

/** Output bounds buffer: MinXYZ, MaxXYZ. */
RWBuffer<float4> OutBounds;

/** Local storage for bounds to be reduced by this workgroup. */
groupshared float3 LocalBounds[2 * THREAD_COUNT];

/** Particle texture dimensions */
int TextureSizeX;
int TextureSizeY;

[numthreads(THREAD_COUNT,1,1)]
void ComputeParticleBounds(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupIdXYZ : SV_GroupID )
{
	uint i;

	// Thread-private particle bounds.
	float3 PrivateBounds[2];

	// Determine group and thread IDs.
	const uint ThreadId = GroupThreadId.x;
	const uint GroupId = GroupIdXYZ.x;

	// Raking index.
	const uint RakingThreadId = ThreadId & (RAKING_THREADS - 1);
	const uint RakingThreadOffset = (ThreadId < RAKING_THREADS) ? 0 : RAKING_THREADS + RAKING_PADDING;
	const uint RakingIndex = RakingThreadId + RakingThreadOffset;

	// Setup chunk indices.
	uint FirstChunkIndex;
	uint ChunkCount;
	if ( GroupId < ParticleBounds.ExtraChunkCount )
	{
		FirstChunkIndex = GroupId * (ParticleBounds.ChunksPerGroup + 1);
		ChunkCount = ParticleBounds.ChunksPerGroup + 1;
	}
	else
	{
		FirstChunkIndex = GroupId * ParticleBounds.ChunksPerGroup + ParticleBounds.ExtraChunkCount;
		ChunkCount = ParticleBounds.ChunksPerGroup;
	}

	// Initialize bounds.
	PrivateBounds[0] = float3( MAX_FLT, MAX_FLT, MAX_FLT );
	PrivateBounds[1] = float3( MIN_FLT, MIN_FLT, MIN_FLT );

	// Buffer offsets.
	uint InputIndex = FirstChunkIndex * THREAD_COUNT + ThreadId;
	uint OutputIndex = 2 * GroupId;

	for ( uint ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex )
	{
		if ( InputIndex < ParticleBounds.ParticleCount )
		{
			// Read in the particle index and its position.
			float3 ParticleIndices = InParticleIndices[InputIndex].xyz;
			const float2 ParticleIndex = ParticleIndices.xy * TilePageScale.xy + GetTilePageOffset(ParticleIndices.z, TilePageScale);
			int3 ParticleTexel = int3(ParticleIndex.xy * int2(TextureSizeX, TextureSizeY), 0);
			const float4 ParticlePosition = PositionTexture.Load(ParticleTexel);
			if ( ParticlePosition.w <= 1.0f )
			{
				PrivateBounds[0] = min( PrivateBounds[0], ParticlePosition.xyz );
				PrivateBounds[1] = max( PrivateBounds[1], ParticlePosition.xyz );
			}
		}
		InputIndex += THREAD_COUNT;
	}

	// Store bounds in shared memory.
	LocalBounds[ThreadId] = PrivateBounds[0];
	LocalBounds[ThreadId + THREAD_COUNT] = PrivateBounds[1];

	// Acquire LocalBounds.
	GroupMemoryBarrierWithGroupSync();

	// Perform thread-local reductions.
	if ( ThreadId < RAKING_THREADS )
	{
		PrivateBounds[0] = LocalBounds[RakingThreadId];
		for ( i = 0; i < THREAD_COUNT / RAKING_THREADS; ++i )
		{
			PrivateBounds[0] = min( PrivateBounds[0], LocalBounds[i * RAKING_THREADS + RakingThreadId] );
		}
	}
	else if ( ThreadId < RAKING_THREADS * 2 )
	{
		PrivateBounds[1] = LocalBounds[RakingThreadId + THREAD_COUNT];
		for ( i = 0; i < THREAD_COUNT / RAKING_THREADS; ++i )
		{
			PrivateBounds[1] = max( PrivateBounds[1], LocalBounds[i * RAKING_THREADS + RakingThreadId + THREAD_COUNT] );
		}
	}

	// Release LocalBounds.
	GroupMemoryBarrierWithGroupSync();

	// Store reductions.
	if ( ThreadId < RAKING_THREADS )
	{
		LocalBounds[RakingThreadId] = PrivateBounds[0];
	}
	else if ( ThreadId < RAKING_THREADS * 2 )
	{
		LocalBounds[RakingThreadId + RAKING_THREADS + RAKING_PADDING] = PrivateBounds[1];
	}

	// Clear raking padding.
	if ( ThreadId < RAKING_PADDING )
	{
		LocalBounds[ThreadId + RAKING_THREADS] = MAX_FLT;
	}
	else if ( ThreadId < RAKING_PADDING * 2 )
	{
		LocalBounds[ThreadId + RAKING_THREADS * 2 + RAKING_PADDING] = MIN_FLT;
	}

	// Acquire LocalBounds.
	GroupMemoryBarrierWithGroupSync();

	// Intra-warp reductions.
	if ( ThreadId < RAKING_THREADS )
	{
		PrivateBounds[0] = LocalBounds[RakingIndex];
		[unroll]
		for ( uint RakingOffset = 1; RakingOffset < RAKING_THREADS; RakingOffset <<= 1 )
		{
			PrivateBounds[0] = min( PrivateBounds[0], LocalBounds[RakingIndex + RakingOffset] );
			LocalBounds[RakingIndex] = PrivateBounds[0];
		}
	}
	else if ( ThreadId < RAKING_THREADS * 2 )
	{
		PrivateBounds[0] = LocalBounds[RakingIndex];
		[unroll]
		for ( uint RakingOffset = 1; RakingOffset < RAKING_THREADS; RakingOffset <<= 1 )
		{
			PrivateBounds[0] = max( PrivateBounds[0], LocalBounds[RakingIndex + RakingOffset] );
			LocalBounds[RakingIndex] = PrivateBounds[0];
		}
	}

	// Acquire LocalBounds.
	GroupMemoryBarrierWithGroupSync();

	// Store bounds for this work group.
	if ( ThreadId < 2 )
	{
		OutBounds[GroupId * 2 + ThreadId] = float4(LocalBounds[ThreadId * (RAKING_THREADS + RAKING_PADDING)],0);
	}
}
