// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleCurveInjectionShader.usf: Shaders for uploading curves to the GPU.
=============================================================================*/

#include "Common.ush"

/*-----------------------------------------------------------------------------
	Shared declarations and functions.
-----------------------------------------------------------------------------*/

struct FShaderInterpolants
{
	/** The curve sample. */
	float4 Sample : TEXCOORD0;
};

/*-----------------------------------------------------------------------------
	Vertex shader.
-----------------------------------------------------------------------------*/
#if VERTEXSHADER

struct FVertexInput
{
	/** The initial position and relative time of the particle. */
	float4 Sample		: ATTRIBUTE0;
	/** The texture coordinate of the corner of the sprite being rendered. */
	float2 TexCoord		: ATTRIBUTE1;
	/** The sample index. */
	uint SampleIndex	: SV_InstanceID;
};

void VertexMain(
	in FVertexInput Input,
	out FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_POSITION
	)
{
	float2 TexelCoord = Input.TexCoord.xy * ParticleCurveInjection.PixelScale.xy +
		ParticleCurveInjection.CurveOffset.xy;
	TexelCoord.x += (float)Input.SampleIndex * ParticleCurveInjection.PixelScale.x;
	OutPosition = float4(
		TexelCoord.xy * float2(2.0f,-2.0f) + float2(-1.0f,1.0f),
		0,
		1
		);

	// Pass sample on to the pixel shader.
	Interpolants.Sample = Input.Sample FCOLOR_COMPONENT_SWIZZLE;
}

#endif // #if VERTEXSHADER

/*-----------------------------------------------------------------------------
	Pixel shader.
-----------------------------------------------------------------------------*/
#if PIXELSHADER

void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutSample : SV_Target0
	)
{
	// Store sample in the texture.
	OutSample = Interpolants.Sample;
}

#endif // #if PIXELSHADER
