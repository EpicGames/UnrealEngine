// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleGPUSpriteVertexFactory.usf: Vertex factory for GPU simulated particles.
=============================================================================*/

#define PARTICLE_SPRITE_FACTORY 1

#include "VertexFactoryCommon.ush"
#include "ParticleVertexFactoryCommon.ush"
#include "ParticleSimulationCommon.ush"

#define USE_PARTICLE_LIGHTING_OFFSET (FEATURE_LEVEL >= FEATURE_LEVEL_SM5 && !MATERIAL_SHADINGMODEL_UNLIT && MATERIAL_DOMAIN_SURFACE)
#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION || PASS_NEEDS_PRIMITIVE_VOLUME_BOUNDS)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)
#define USE_SPHERICAL_NORMALS (GENERATE_SPHERICAL_PARTICLE_NORMALS && USE_PARTICLE_POSITION)
#define USE_PARTICLE_SIZE (NEEDS_PARTICLE_SIZE)
#define USE_PARTICLE_SPRITE_ROTATION (NEEDS_PARTICLE_SPRITE_ROTATION)

/** Buffer containing particle indices. */
Buffer<float4> ParticleIndices;

/** Offset in to the particle indices buffer. */
uint ParticleIndicesOffset;
float3 TilePageScale;


/** Texture containing positions for all particles. */
Texture2D PositionTexture;
SamplerState PositionTextureSampler;
Texture2D VelocityTexture;
SamplerState VelocityTextureSampler;
Texture2D AttributesTexture;
SamplerState AttributesTextureSampler;
Texture2D CurveTexture;
SamplerState CurveTextureSampler;

#undef GetInstanceIdFromVF
#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)

/**
 * Vertex attributes to fetch.
 */
struct FVertexFactoryInput
{
	/** Unique vertex ID. */
	uint VertexId			: SV_VertexID;
	/** Unique instance ID. */
	uint InstanceId			: SV_InstanceID;
	/** Per-vertex: texture coordinate. */
	float2 TexCoord			: ATTRIBUTE0;
	
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
};

/**
 * Attributes to interpolate from the vertex shader to the pixel shader.
 */
struct FVertexFactoryInterpolantsVSToPS
{
#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2]	: TEXCOORD0;
#endif

#if USE_PARTICLE_SUBUVS
	float4 ParticleSubUVs : PARTICLE_SUBUVS;
#endif

#if NEEDS_PARTICLE_RANDOM
	/** X: SubImageLerp Y:RelativeTime Z:MotionBlurFade W:Random */
	float4 Misc						: TEXCOORD4;
#elif USE_PARTICLE_SUBUVS || USE_PARTICLE_TIME || USES_PARTICLE_MOTION_BLUR
	/** X: SubImageLerp Y:RelativeTime Z:MotionBlurFade */
	float3 Misc						: TEXCOORD4;
#endif

#if NEEDS_PARTICLE_COLOR
	/** Sprite color. */
	float4 Color					: TEXCOORD5;
#endif

#if USE_PARTICLE_POSITION
	/** Particle center in translated world space and radius */
	float4 ParticleTranslatedWorldPositionAndSize	: PARTICLE_POSITION;
#endif

	/** The velocity of the particle, XYZ: direction, W: speed. */
#if USE_PARTICLE_VELOCITY
	float4 ParticleVelocity			: PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset : PARTICLE_LIGHTING_OFFSET;
#endif

#if !USE_SPHERICAL_NORMALS
	float4 TangentToWorld0 : TANGENTX;
	float4 TangentToWorld2 : TANGENTZ;
#endif
	
	/** The size of the particle. */
#if USE_PARTICLE_SIZE
	float2 ParticleSize			: PARTICLE_SIZE;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	float ParticleSpriteRotation : PARTICLE_SPRITE_ROTATION;
#endif
};

/**
 * Intermediate values computed in the vertex shader.
 */
struct FVertexFactoryIntermediates
{
	/** The position of the particle in translated world space. */
	float3 ParticleTranslatedWorldPosition;
	/** The position of the vertex in translated world space. */
	float3 VertexWorldPosition;
	/** The position of the vertex in local instance space. */
	float3 PositionInstanceSpace;

#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset;
#endif

	/** The texture coordinate at this vertex. */
	float4 TexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3 //Unflipped UVs in texcoords 3 and 4
	float4 TexCoord_Unflipped;
#endif
	/** The sprite tangent in world space (+V). */
	float3 TangentUp;
	/** The sprite tangent in world space (+U). */
	float3 TangentRight;
	/** The color of the sprite. */
	float4 Color;
	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity;
	/** The sub-image lerp. */
	float SubImageLerp;
	/** Relative time. */
	float RelativeTime;
	/** Amount to fade the sprite due to camera motion blur. */
	float MotionBlurFade;
	/** Random value per particle [0-1]. */
	float Random;
	/** Radius of the particle when represented by a sphere. */
	float ParticleRadius;
	/** Size of the particle. */
	float2 ParticleSize;
#if USE_PARTICLE_SPRITE_ROTATION
	float ParticleSpriteRotation;
#endif
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

float3 SafeNormalize(float3 V)
{
	return V / sqrt(max(dot(V,V),0.01));
}

/**
 * Compute the tangents for a sprite given a rotation.
 * @param OutUp - The tangent vector pointing up in screen space.
 * @param OutRight - The tangent vector pointing right in screen space.
 * @param Rotation - The rotation of the sprite.
 * @param ParticleTranslatedWorldPosition - Translated world position of the particle.
 * @param ParticleWorldDirection - World space direction in which the particle is traveling.
 */
void GetSpriteTangents(
	out float3 OutUp,
	out float3 OutRight,
	float Rotation,
	float3 ParticleTranslatedWorldPosition,
	float3 ParticleWorldDirection )
{
	// Select camera up/right vectors.
	float3 ResolvedViewRight = lerp(ResolvedView.ViewRight, ResolvedView.HMDViewNoRollRight, EmitterUniforms.RemoveHMDRoll);
	float3 ResolvedViewUp = lerp(ResolvedView.ViewUp, ResolvedView.HMDViewNoRollUp, EmitterUniforms.RemoveHMDRoll);

	float3 CameraRight = lerp(ResolvedViewRight, EmitterDynamicUniforms.AxisLockRight.xyz, EmitterDynamicUniforms.AxisLockRight.w);
	float3 CameraUp = lerp(-ResolvedViewUp, EmitterDynamicUniforms.AxisLockUp.xyz, EmitterDynamicUniforms.AxisLockUp.w);

	// Determine the vector from the particle to the camera and the particle's movement direction.
	float3 CameraDirection	= -GetCameraVectorFromTranslatedWorldPosition(ResolvedView, ParticleTranslatedWorldPosition.xyz);
	float3 RightVector = CameraRight.xyz;
	float3 UpVector = CameraUp.xyz;

	BRANCH
	if (EmitterUniforms.CameraFacingBlend.x > 0.f)
	{
		// Blend between PSA_FacingCamera and PSA_Square over distance
		float CameraDistanceSq = GetDistanceToCameraFromViewVectorSqr(ResolvedView.TranslatedWorldCameraOrigin.xyz - ParticleTranslatedWorldPosition.xyz);
		float AlignmentMode = saturate(CameraDistanceSq * EmitterUniforms.CameraFacingBlend.y - EmitterUniforms.CameraFacingBlend.z);

		float3 CameraFacingRight = SafeNormalize(cross(CameraDirection, float3(0,0,1)));
		float3 CameraFacingUp = cross(CameraDirection, CameraFacingRight);

		RightVector = normalize(lerp(RightVector, CameraFacingRight, AlignmentMode));
		UpVector = normalize(lerp(UpVector, CameraFacingUp, AlignmentMode));
	}
	else
	{
		FLATTEN
		if (EmitterUniforms.TangentSelector.y > 0)
		{
			// Tangent vectors for PSA_Velocity.
			RightVector	= SafeNormalize(cross(CameraDirection, ParticleWorldDirection));
			UpVector = -ParticleWorldDirection;
		}
		else if (EmitterUniforms.TangentSelector.z > 0)
		{
			// Tangent vectors for rotation locked about an axis.
			RightVector = EmitterDynamicUniforms.AxisLockRight.xyz;
			UpVector = -SafeNormalize(cross(RightVector,CameraDirection));
		}
		else if (EmitterUniforms.TangentSelector.w > 0)
		{
			// Tangent vectors for camera facing position.
			RightVector = SafeNormalize(cross(CameraDirection,float3(0,0,1)));
			UpVector = cross(CameraDirection, RightVector);
		}
	}

	// Determine the angle of rotation.
	float SinRotation; // = 0
	float CosRotation; // = 1
	const float SpriteRotation = Rotation + EmitterUniforms.RotationBias;
	sincos(SpriteRotation, SinRotation, CosRotation);

	// Rotate the sprite to determine final tangents.
	OutRight	= SinRotation * UpVector + CosRotation * RightVector;
	OutUp		= CosRotation * UpVector - SinRotation * RightVector;
}

/**
 * Computes intermediates for the given vertex.
 * @param Input - Vertex attributes.
 * @returns the computed intermediate values.
 */
FVertexFactoryIntermediates GetVertexFactoryIntermediates( FVertexFactoryInput Input )
{
	// Sample the position and velocity textures to get the current state of the particle.
	uint InstanceId = GetInstanceId(Input.InstanceId) * PARTICLES_PER_INSTANCE + Input.VertexId / 4;
	float3 ParticleIndex = ParticleIndices[ParticleIndicesOffset + InstanceId].xyz;
	ParticleIndex.xy = ParticleIndex.xy * TilePageScale.xy + GetTilePageOffset(ParticleIndex.z, TilePageScale.xy, TilePageScale.z);
	float4 PositionSample = Texture2DSampleLevel(PositionTexture, PositionTextureSampler, ParticleIndex.xy, 0);
	float4 VelocitySample = Texture2DSampleLevel(VelocityTexture, VelocityTextureSampler, ParticleIndex.xy, 0);
	float4 AttributeSample = Texture2DSampleLevel(AttributesTexture, AttributesTextureSampler, ParticleIndex.xy, 0);

	// For clarity, store some information in local variables.
	const float RelativeTime = PositionSample.w;
	const float IsAlive = step( RelativeTime, 1.0f );

	// Put velocity in to world space.
	float3 ParticleWorldVelocity = mul(VelocitySample.xyz, GetLocalToWorld3x3()).xyz;

	// Add a small bias to the direction to prevent issues when velocity is zero.
	float3 ParticleDirection = normalize(ParticleWorldVelocity.xyz + float3(0,0,0.0001f));
	float ParticleSpeed = length(ParticleWorldVelocity.xyz);

	// Sample the color curve.
	const float4 InitialColor = float4(1,1,1,1);
	float2 ColorCurveTexCoord = EmitterUniforms.ColorCurve.xy +
		EmitterUniforms.ColorCurve.zw * RelativeTime;
	float4 ColorCurveSample = Texture2DSampleLevel(CurveTexture, CurveTextureSampler, ColorCurveTexCoord, 0 );
	float4 ColorScale = ColorCurveSample * EmitterUniforms.ColorScale + EmitterUniforms.ColorBias;
	float4 Color = ColorScale * InitialColor * EmitterDynamicUniforms.DynamicColor;

	// Sample the curve containing misc. attributes.
	float2 MiscCurveTexCoord = EmitterUniforms.MiscCurve.xy +
		EmitterUniforms.MiscCurve.zw * RelativeTime;
	float4 MiscCurveSample = Texture2DSampleLevel(CurveTexture, CurveTextureSampler, MiscCurveTexCoord, 0 );
	float4 MiscCurve = MiscCurveSample * EmitterUniforms.MiscScale + EmitterUniforms.MiscBias;

	// Compute the size of the sprite. Note it is (0,0) if the sprite is dead.
	// Reconstruct the real size after we've encoded the UV flip mode as sizes > 0.5;
	float2 UVFlipOffset = float2( AttributeSample.x < 0.5 ? 0.0f : -0.5, AttributeSample.y < 0.5 ? 0.0f : -0.5);
	float2 InitialSize = (AttributeSample.xy + UVFlipOffset) * float2(2.0,2.0);
	float2 SizeScale = MiscCurve.xy * EmitterDynamicUniforms.LocalToWorldScale;

	//Explicit ordering of clamp's min max to bypass compiler diff on PS4.
	float2 SizeScaleBySpeed = max(EmitterUniforms.SizeBySpeed.xy * ParticleSpeed,float2(1,1));
	SizeScaleBySpeed = min(SizeScaleBySpeed, EmitterUniforms.SizeBySpeed.zw);

	float2 Size = InitialSize * SizeScale * SizeScaleBySpeed * IsAlive.xx;
	
	float2 FlippedUV = Input.TexCoord.xy;
	FlippedUV.x = UVFlipOffset.x == 0.0 ? 1.0 - FlippedUV.x : FlippedUV.x;
	FlippedUV.y = UVFlipOffset.y == 0.0 ? 1.0 - FlippedUV.y : FlippedUV.y;

	// SubUV.
	float SubImageIndex = MiscCurve.z;
	float SubImageLerp = frac(SubImageIndex);
	float SubImageA = SubImageIndex - SubImageLerp;
	float SubImageB = SubImageA + 1;
	float SubImageAH = fmod( SubImageA, EmitterUniforms.SubImageSize.x );
	float SubImageBH = fmod( SubImageB, EmitterUniforms.SubImageSize.x );
	float SubImageAV = floor( SubImageA * EmitterUniforms.SubImageSize.z );
	float SubImageBV = floor( SubImageB * EmitterUniforms.SubImageSize.z );

	// Compute the final texture coordinates for both subimages.
	float4 TexCoord; 
	TexCoord.xy = (float2( SubImageAH, SubImageAV ) + FlippedUV) * EmitterUniforms.SubImageSize.zw;
	TexCoord.zw = (float2(SubImageBH, SubImageBV) + FlippedUV) * EmitterUniforms.SubImageSize.zw;

#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3 //Unflipped UVs in texcoords 3 and 4
	float4 TexCoord_Unflipped;
	TexCoord_Unflipped.xy = (float2(SubImageAH, SubImageAV) + Input.TexCoord.xy) * EmitterUniforms.SubImageSize.zw;
	TexCoord_Unflipped.zw = (float2(SubImageBH, SubImageBV) + Input.TexCoord.xy) * EmitterUniforms.SubImageSize.zw;
#endif

	// Current rotation of the sprite. Map [0,1] to one full rotation.
	float RotationRate = AttributeSample.w * EmitterUniforms.RotationRateScale;
	float Rotation = AttributeSample.z + RotationRate * RelativeTime;
	Rotation = Rotation * 2.0f * 3.1415926535897932f;

	// Transform position to (post-view-translation) world space.
	FLWCMatrix Transform = DFToTileOffset(GetPrimitiveDataFromUniformBuffer().LocalToWorld);
	FLWCVector3 LWCTileOffset = MakeLWCVector3(EmitterDynamicUniforms.LWCTile, 0);
	FLWCVector3 ParticleWorldPosition = LWCAdd(LWCMultiply(PositionSample.xyz, Transform), LWCTileOffset);
	float3 ParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(ParticleWorldPosition, ResolvedView.TileOffset.PreViewTranslation));

	// Compute tangents for the sprite.
	float3 TangentUp, TangentRight;
	GetSpriteTangents( TangentUp, TangentRight, Rotation, ParticleTranslatedWorldPosition, ParticleDirection );

	// Offset of the sprite from the particle's location.
	float3 VertexWorldOffset = Size.x * (Input.TexCoord.x + EmitterUniforms.PivotOffset.x) * TangentRight
		+ Size.y * (Input.TexCoord.y + EmitterUniforms.PivotOffset.y) * TangentUp;

	// Optional camera motion blur.
#if USES_PARTICLE_MOTION_BLUR
	float3 MblurVector = -View.WorldCameraMovementSinceLastFrame;
	float3 MblurDirection = normalize(MblurVector + 0.0001f);
	//float MblurFactor = saturate(dot(MblurDirection,normalize(VertexWorldOffset)) * 100000.0f);
	float MblurFactor = clamp(dot(MblurDirection,normalize(VertexWorldOffset)) * 100000.0f,-1.0f,1.0f);
	VertexWorldOffset += (MblurFactor * MblurVector * EmitterUniforms.CameraMotionBlurAmount);
#endif // #if USES_PARTICLE_MOTION_BLUR

	// Determine the world position of this vertex (one corner of the sprite).
	float3 VertexWorldPosition = ParticleTranslatedWorldPosition + VertexWorldOffset;

	// Build and return the set of intermediates.
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.ParticleTranslatedWorldPosition = ParticleTranslatedWorldPosition;
	Intermediates.VertexWorldPosition = VertexWorldPosition;
	Intermediates.PositionInstanceSpace = PositionSample.xyz;

#if USE_SPHERICAL_NORMALS
	// Recenter the particle position for GENERATE_SPHERICAL_PARTICLE_NORMALS to use the right center.
	Intermediates.ParticleTranslatedWorldPosition += Size.x * (EmitterUniforms.PivotOffset.x + .5) * TangentRight + Size.y * (EmitterUniforms.PivotOffset.y + .5) * TangentUp;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	// Hash function based on the particle ID to generate a uniformly distributed 3d offset
	float3 RandomParticleOffset = frac((ParticleIndex.x + 10) * (ParticleIndex.y + 10) * float3(1341.456345, 2633.578, 5623.983)) * 2 - 1;
	Intermediates.LightingPositionOffset = .5 * RandomParticleOffset;
#endif

	Intermediates.TexCoord = TexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3 //Unflipped UVs in texcoords 3 and 4
	Intermediates.TexCoord_Unflipped = TexCoord_Unflipped; 
#endif
	Intermediates.TangentUp = TangentUp;
	Intermediates.TangentRight = TangentRight;
	Intermediates.Color = Color;
	Intermediates.ParticleVelocity = float4(ParticleDirection, ParticleSpeed);
	Intermediates.SubImageLerp = SubImageLerp;
	Intermediates.RelativeTime = RelativeTime;
#if USES_PARTICLE_MOTION_BLUR
	Intermediates.MotionBlurFade = MblurFactor;
#else // #if USES_PARTICLE_MOTION_BLUR
	Intermediates.MotionBlurFade = 0.0f;
#endif // #if USES_PARTICLE_MOTION_BLUR
#if NEEDS_PARTICLE_RANDOM
	Intermediates.Random = PseudoRandom(ParticleIndex.xy + float2(EmitterDynamicUniforms.EmitterInstRandom,EmitterDynamicUniforms.EmitterInstRandom));
#else
	Intermediates.Random = 0.0f;
#endif

	Intermediates.ParticleRadius = .5f * min(Size.x, Size.y);
	Intermediates.ParticleSize = Size;

#if USE_PARTICLE_SPRITE_ROTATION
	Intermediates.ParticleSpriteRotation = Rotation;
#endif

	return Intermediates;
}

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndex % 2 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndex % 2)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

/**
 * Computes material parameterss for a given pixel.
 * @param Interpolants - Attributes interpolated from the vertex shader.
 * @returns per-pixel material parameters.
 */
FMaterialPixelParameters GetMaterialPixelParameters( FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition )
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();
	Result.Particle.MacroUV = EmitterDynamicUniforms.MacroUVParameters;

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.Misc.y;
#endif

#if USES_PARTICLE_MOTION_BLUR
	Result.Particle.MotionBlurFade = 1.0f - abs(Interpolants.Misc.z);
#else // #if USES_PARTICLE_MOTION_BLUR
	Result.Particle.MotionBlurFade = 1.0f;
#endif // #if USES_PARTICLE_MOTION_BLUR

#if NEEDS_PARTICLE_RANDOM
	Result.Particle.Random = Interpolants.Misc.w;
#else
	Result.Particle.Random = 0.0f;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Interpolants.ParticleTranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_SPHERICAL_NORMALS
	// can be optimized
	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);

		Result.TangentToWorld = GetSphericalParticleNormal(TranslatedWorldPosition, Interpolants.ParticleTranslatedWorldPositionAndSize.xyz, Interpolants.ParticleTranslatedWorldPositionAndSize.w);
	}
#else
	half3 TangentToWorld0 = Interpolants.TangentToWorld0.xyz;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	Result.TangentToWorld = AssembleTangentToWorld(TangentToWorld0, TangentToWorld2);
#endif

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.Color;
#endif

	Result.VertexColor = 1;
	Result.TwoSidedSign = 1;

#if USE_PARTICLE_LIGHTING_OFFSET
	Result.LightingPositionOffset = Interpolants.LightingPositionOffset;
#endif

#if USE_PARTICLE_SUBUVS
	Result.Particle.SubUVCoords[0] = Interpolants.ParticleSubUVs.xy;
	Result.Particle.SubUVCoords[1] = Interpolants.ParticleSubUVs.zw;
	Result.Particle.SubUVLerp = Interpolants.Misc.x;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Interpolants.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Interpolants.ParticleSpriteRotation;
#endif
	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/**
 * Computes material parameters for a given vertex.
 * @param Input - Attributes for this vertex.
 * @param Intermediates - Intermediates computed for this vertex.
 * @param WorldPosition - The position of this vertex in world space.
 * @param TangentToLocal - The tangent basis for this vertex.
 * @returns per-vertex material parameters.
 */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = 1;
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3()); 
	Result.Particle.MacroUV = EmitterDynamicUniforms.MacroUVParameters;
	Result.PreSkinnedPosition = WorldPosition.xyz;
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

	// Previous frame not handled deliberately. Lacks necessary information and
	// primitives using this VF are usually transparent and hence don't output velocity
	Result.PrevFrameLocalToWorld = GetPrimitiveDataFromUniformBuffer().LocalToWorld;

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Intermediates.RelativeTime;
#endif

#if USES_PARTICLE_MOTION_BLUR
	Result.Particle.MotionBlurFade = 1.0f - abs(Intermediates.MotionBlurFade);
#else // #if USES_PARTICLE_MOTION_BLUR
	Result.Particle.MotionBlurFade = 1.0f;
#endif // #if USES_PARTICLE_MOTION_BLUR

#if NEEDS_PARTICLE_RANDOM
	Result.Particle.Random = Intermediates.Random;
#else
	Result.Particle.Random = 0.0f;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = float4(Intermediates.ParticleTranslatedWorldPosition, Intermediates.ParticleRadius);
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

	Result.Particle.Color = Intermediates.Color;
	
#if NUM_MATERIAL_TEXCOORDS_VERTEX
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 1
		Result.TexCoords[0] = Intermediates.TexCoord.xy;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
		Result.TexCoords[1] = Intermediates.TexCoord.zw;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
		Result.TexCoords[2] = Intermediates.TexCoord_Unflipped.xy;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
		Result.TexCoords[3] = Intermediates.TexCoord_Unflipped.zw;
#endif	// >= 4
#endif	// >= 3
#endif	// >= 2
#endif	// >= 1
#endif	//NUM_MATERIAL_TEXCOORDS

#if USE_PARTICLE_SUBUVS
		Result.Particle.SubUVCoords[0] = Intermediates.TexCoord.xy;
		Result.Particle.SubUVCoords[1] = Intermediates.TexCoord.zw;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Intermediates.ParticleSpriteRotation;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

/**
 * Computes the world space position of this vertex.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the position of this vertex in world space.
 */
float4 VertexFactoryGetWorldPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return float4(Intermediates.VertexWorldPosition,1);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.PositionInstanceSpace;
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 TranslatedWorldPosition)
{
#if SPHERICAL_PARTICLE_OPACITY
	// For particles using spherical opacity, move the quad toward the viewer so that it lies in front of the sphere defined by the particle
	// This avoids opaque objects intersecting the particle from causing clipping artifacts due to depth testing
	// The downside is that the particle will clip the near plane sooner

	float Radius = Intermediates.ParticleRadius;
	return ReprojectPosition(TranslatedWorldPosition, Radius);
#else
	return TranslatedWorldPosition;
#endif
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

/**
 * Computes the tangent basis for this vertex in world space.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the tangent basis for this vertex in world space.
 */
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3x3 TangentToLocal;
	TangentToLocal[0] = Intermediates.TangentRight;
	TangentToLocal[1] = Intermediates.TangentUp;
	TangentToLocal[2] = normalize(cross(Intermediates.TangentRight.xyz , Intermediates.TangentUp.xyz));
	return TangentToLocal;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return VertexFactoryGetTangentToLocal(Input, Intermediates)[2].xyz;
}

/**
 * Constructs values that need to be interpolated from the vertex shader to the pixel shader.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns interpolants.
 */
FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters )
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

#if USE_PARTICLE_SUBUVS
	Interpolants.ParticleSubUVs.xy = VertexParameters.Particle.SubUVCoords[0];
	Interpolants.ParticleSubUVs.zw = VertexParameters.Particle.SubUVCoords[1];
#endif

#if USE_PARTICLE_SUBUVS || USE_PARTICLE_TIME || USES_PARTICLE_MOTION_BLUR || NEEDS_PARTICLE_RANDOM
	Interpolants.Misc.x = Intermediates.SubImageLerp;
	Interpolants.Misc.y = Intermediates.RelativeTime;
	Interpolants.Misc.z = Intermediates.MotionBlurFade;
#endif

#if NEEDS_PARTICLE_RANDOM
	Interpolants.Misc.w = Intermediates.Random;
#endif

#if NEEDS_PARTICLE_COLOR
	Interpolants.Color = Intermediates.Color;
#endif

#if USE_PARTICLE_POSITION
	Interpolants.ParticleTranslatedWorldPositionAndSize = float4(Intermediates.ParticleTranslatedWorldPosition, Intermediates.ParticleRadius);
#endif

#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	Interpolants.LightingPositionOffset = Intermediates.LightingPositionOffset;
#endif

#if !USE_SPHERICAL_NORMALS
	// Note that "local" space for particles is actually oriented in world space! Therefore no rotation is needed.
	float3x3 TangentToWorld = VertexFactoryGetTangentToLocal(Input, Intermediates);

	float3 TangentToWorld0 = TangentToWorld[0];
	float4 TangentToWorld2 = float4(TangentToWorld[2], sign(determinant(TangentToWorld)));

	Interpolants.TangentToWorld0 = float4(TangentToWorld0,0);
	Interpolants.TangentToWorld2 = TangentToWorld2;
#endif

#if USE_PARTICLE_SIZE
	Interpolants.ParticleSize = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Interpolants.ParticleSpriteRotation = Intermediates.ParticleSpriteRotation;
#endif

	return Interpolants;
}

/**
 * Computes the position of this vertex last frame in world space.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the previous position of this vertex in world space.
 */
float4 VertexFactoryGetPreviousWorldPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3 Position = Intermediates.VertexWorldPosition;
	Position.xyz -= Intermediates.ParticleVelocity.xyz * EmitterUniforms.UseVelocityForMotionBlur;
	return float4(Position, 1);
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.PositionInstanceSpace;
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if USE_PARTICLE_POSITION
	return Interpolants.ParticleTranslatedWorldPositionAndSize;
#endif
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#include "VertexFactoryDefaultInterface.ush"