// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleInjectionShader.usf: Shaders for uploading particles to the GPU.
=============================================================================*/

#include "Common.ush"
#include "ParticleSimulationCommon.ush"

/*-----------------------------------------------------------------------------
	Shared declarations and functions.
-----------------------------------------------------------------------------*/

struct FShaderInterpolants
{
	/** The initial position and relative time of the particle. */
	float4 InitialPosition		: TEXCOORD0;
	/** The initial velocity and time scale of the particle. */
	float4 InitialVelocity		: TEXCOORD1;
	/** The render attributes for the particle. */
	float4 RenderAttributes		: TEXCOORD2;
	/** The simulation attributes for the particle. */
	float4 SimulationAttributes : TEXCOORD3;
};

/*-----------------------------------------------------------------------------
	Vertex shader.
-----------------------------------------------------------------------------*/
#if VERTEXSHADER

struct FVertexInput
{
	/** The initial position and relative time of the particle. */
	float4 InitialPosition		: ATTRIBUTE0;
	/** The initial velocity and time scale of the particle. */
	float4 InitialVelocity		: ATTRIBUTE1;
	/** The render attributes for the particle. */
	float4 RenderAttributes		: ATTRIBUTE2;
	/** The simulation attributes for the particle. */
	float4 SimulationAttributes : ATTRIBUTE3;
	/** The offset in to the texture to store the particle. */
	float3 ParticleIndex	: ATTRIBUTE4;
	/** The texture coordinate of the corner of the sprite being rendered. */
	float2 TexCoord				: ATTRIBUTE5;
};

void VertexMain(
	in FVertexInput Input,
	out FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_POSITION
	)
{
	float2 ParticleCoord = Input.TexCoord.xy * ParticleInjection.TexCoordScale.xy + Input.ParticleIndex.xy * ParticleInjection.TilePageScale.xy + GetTilePageOffset(Input.ParticleIndex.z, ParticleInjection.TilePageScale, true);
	OutPosition = float4(
		ParticleCoord.xy * float2(2.0f,-2.0f) + float2(-1.0f,1.0f),
		0,
		1
		);

	// Pass initial values on to the pixel shader.
	Interpolants.InitialPosition = Input.InitialPosition;
	Interpolants.InitialVelocity = Input.InitialVelocity;
	Interpolants.RenderAttributes = Input.RenderAttributes;
	Interpolants.SimulationAttributes = Input.SimulationAttributes;
}

#endif // #if VERTEXSHADER

/*-----------------------------------------------------------------------------
	Particle simulation pixel shader.
-----------------------------------------------------------------------------*/
#if PIXELSHADER

#if STATIC_PROPERTIES_ONLY
void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutRenderAttributes : SV_Target0,
	out float4 OutSimulationAttributes : SV_Target1
)
{
	// Store attributes in textures.
	OutRenderAttributes = Interpolants.RenderAttributes;
	OutSimulationAttributes = Interpolants.SimulationAttributes;
}
#else
void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutPosition				: SV_Target0,
	out float4 OutVelocity				: SV_Target1,
	out float4 OutRenderAttributes		: SV_Target2,
	out float4 OutSimulationAttributes	: SV_Target3
	)
{
	// Store attributes in textures.
	OutPosition = Interpolants.InitialPosition;
	OutVelocity = Interpolants.InitialVelocity;
	OutRenderAttributes = Interpolants.RenderAttributes;
	OutSimulationAttributes = Interpolants.SimulationAttributes;
}
#endif

#endif // #if PIXELSHADER
