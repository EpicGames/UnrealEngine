// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleSimVisualizeShader.usf: Shader for visualizing particle simulation.
=============================================================================*/

#include "Common.ush"

/*-----------------------------------------------------------------------------
	Shared declarations and functions.
-----------------------------------------------------------------------------*/

struct FShaderInterpolants
{
	float2 TexCoord : TEXCOORD0;
};

/*-----------------------------------------------------------------------------
	Vertex shader.
-----------------------------------------------------------------------------*/
#if VERTEXSHADER

struct FVertexInput
{
	float2 TexCoord : ATTRIBUTE0;
};

void VertexMain(
	in FVertexInput Input,
	out FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_POSITION
	)
{
	float4 ScaleBias = PSV.ScaleBias;
	OutPosition = float4(
		Input.TexCoord.xy * ScaleBias.xy + ScaleBias.zw,
		0,
		1
		);
	Interpolants.TexCoord.x = Input.TexCoord.x;
	Interpolants.TexCoord.y = 1.0f - Input.TexCoord.y;
}

#endif // #if VERTEXSHADER

/*-----------------------------------------------------------------------------
	Pixel shader.
-----------------------------------------------------------------------------*/
#if PIXELSHADER

Texture2D PositionTexture;
SamplerState PositionTextureSampler;
Texture2D CurveTexture;
SamplerState CurveTextureSampler;
int VisualizationMode;

void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutColor : SV_Target0
	)
{
	float4 Color = float4(0,0,0,1);

	if (VisualizationMode == 1)
	{
		float4 Position = Texture2DSample(PositionTexture, PositionTextureSampler, Interpolants.TexCoord.xy);

		// Position.w holds the relative time of the particle.
		const float RelativeTime = Position.w;

		// Relative time should always be >= 0.
		const float IsValid = step(0.0f, RelativeTime);

		// The particle is alive as long as relative time <= 1.0
		const float IsAlive = step( RelativeTime, 1.0f );
		const float IsDead = 1.0f - IsAlive;

		Color = float4(
			IsAlive * IsValid,
			0.0f,
			saturate(1.0f - RelativeTime),
			1
			);
	}
	else if (VisualizationMode == 2)
	{
		//float4 CurveSample = Texture2DSample(PositionTexture, PositionTextureSampler, Interpolants.TexCoord.xy);
		float4 CurveSample = Texture2DSampleLevel(CurveTexture, CurveTextureSampler, Interpolants.TexCoord.xy, 0);
		Color.rgb = CurveSample.rgb;
	}

	OutColor = Color;
}

#endif // #if PIXELSHADER
