// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleSimulationCommon.usf: Particle Simulation shared functions and defines.
=============================================================================*/

#pragma once

#include "MortonCode.ush"

float2 GetTilePageOffset(float ZTileIndex, float2 PageScale, bool bResize = true)
{
	float2 UV  = 0.0f;

	if(bResize)
	{
		uint2 PageOffset = MortonDecode(ZTileIndex);
		UV = PageOffset * PageScale;
	}

	return UV;
}