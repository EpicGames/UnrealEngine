// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleSimulationShader.usf: Shaders for simulating particles on the GPU.
==============================================================================*/

#include "Common.ush"
#include "SceneTexturesCommon.ush"
#include "ParticleSimulationCommon.ush"

#if DISTANCE_FIELD_COLLISION
	#include "DistanceField/GlobalDistanceFieldShared.ush"
#endif

#define IMPROVED_DEPTH_BUFFER_COLLISION 1
#define FORWARD_SHADING_ACCURATE_NORMAL 1

/*------------------------------------------------------------------------------
	Shared declarations and functions.
------------------------------------------------------------------------------*/

struct FShaderInterpolants
{
	/** The texture coordinate at which to sample. */
	float2 TexCoord : TEXCOORD0;
};

/*------------------------------------------------------------------------------
	Vertex shader.
------------------------------------------------------------------------------*/
#if VERTEXSHADER

float2 TexCoordScale;
float2 TilePageScale;

struct FVertexInput
{
	/** The texture coordinate. */
	float2 TexCoord	: ATTRIBUTE0;
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	/** Unique vertex ID. */
	uint VertexId	: SV_VertexID;
	/** Unique instance ID. */
	uint InstanceId	: SV_InstanceID;
#else // Mobile
	/** Tile offsets in per-instance data */
	float4 TileOffsets : ATTRIBUTE1;
#endif
};

/** Buffer from which to read tile offsets. */
Buffer<float4> TileOffsets;

void VertexMain(
	in FVertexInput Input,
	out FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_POSITION
	)
{
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	uint InstanceId = Input.InstanceId * TILES_PER_INSTANCE + Input.VertexId / 4;
	float3 TileOffset = TileOffsets[InstanceId].xyz;
#else
	float3 TileOffset = Input.TileOffsets.xyz;
#endif
	
	float2 TileCoord = Input.TexCoord.xy * TexCoordScale.xy + TileOffset.xy * TilePageScale.xy + GetTilePageOffset(TileOffset.z, TilePageScale, true);

	OutPosition = float4(
		TileCoord.xy * float2(2.0f,-2.0f) + float2(-1.0f,1.0f),
		0,
		1
		);
	Interpolants.TexCoord.xy = TileCoord.xy;
}

#endif // #if VERTEXSHADER

/*------------------------------------------------------------------------------
	Particle simulation pixel shader.
------------------------------------------------------------------------------*/
#if PARTICLE_SIMULATION_PIXELSHADER

/** Define to 1 to force no collision in the shader. */
#define FORCE_NO_COLLISION 0

#if FORCE_NO_COLLISION
	#undef DEPTH_BUFFER_COLLISION
	#define DEPTH_BUFFER_COLLISION 0
	#undef DISTANCE_FIELD_COLLISION
	#define DISTANCE_FIELD_COLLISION 0
#endif

#define FORWARD_SHADING_NORMAL_CALC	(FORWARD_SHADING || ((FEATURE_LEVEL == FEATURE_LEVEL_ES3_1) && VULKAN_PROFILE))

/** Position (XYZ) and squared radius (W) of the point attractor. */
float4 PointAttractor;
/** Position offset (XYZ) to add to particles and strength of the attractor (W). */
float4 PositionOffsetAndAttractorStrength;
/** Amount by which to scale bounds for collision purposes. */
float2 LocalToWorldScale;
/** Amount of time by which to simulate particles. */
float DeltaSeconds;
/** Number of iterations, each applying DeltaSeconds. */ 
int	NumIterations;
/** LWC tile offset, will be 0,0,0 for localspace emitters. */
float3 LWCTile;

/** Texture from which to read particle position. */
Texture2D PositionTexture;
SamplerState PositionTextureSampler;

/** Texture from which to read particle velocity. */
Texture2D VelocityTexture;
SamplerState VelocityTextureSampler;
/** Texture from which to read particle attributes. */
Texture2D AttributesTexture;
SamplerState AttributesTextureSampler;
/** Texture from which curves can be sampled. */
Texture2D CurveTexture;
SamplerState CurveTextureSampler;

/** Textures from which to sample vector forces. */
#if MAX_VECTOR_FIELDS != 4 && MAX_VECTOR_FIELDS != 1
#error This must match MAX_VECTOR_FIELDS in C++ land
#endif
Texture3D VectorFieldTextures_0;

#if MAX_VECTOR_FIELDS > 1
Texture3D VectorFieldTextures_1;
Texture3D VectorFieldTextures_2;
Texture3D VectorFieldTextures_3;
#endif

SamplerState VectorFieldTexturesSampler_0;
#if MAX_VECTOR_FIELDS > 1
SamplerState VectorFieldTexturesSampler_1;
SamplerState VectorFieldTexturesSampler_2;
SamplerState VectorFieldTexturesSampler_3;
#endif

/**
 * Computes the orbit velocity to apply to the particle based on time.
 * @param Time - The time at which to evaluate the velocity.
 * @param RandomOrbit - Random value used to add variation to orbit.
 */
float3 ComputeOrbitVelocity(float Time, float RandomOrbit)
{
	float3 Sines, Cosines;

	// Read parameters.
	const float3 Offset = Simulation.OrbitOffsetBase.xyz + Simulation.OrbitOffsetRange.xyz * RandomOrbit;
	const float3 Frequency = Simulation.OrbitFrequencyBase.xyz + Simulation.OrbitFrequencyRange.xyz * RandomOrbit;
	const float3 Phase = Simulation.OrbitPhaseBase.xyz + Simulation.OrbitPhaseRange.xyz * RandomOrbit;

	// Compute angles along with cos + sin of those angles.
	const float3 Angles = Frequency.xyz * Time.xxx + Phase.xyz;
	sincos(Angles, Sines, Cosines);

	// Compute velocity required to follow orbit path.
	return Offset.xyz * (Frequency.zxy * Cosines.zxy - Frequency.yzx * Sines.yzx);
}

/**
 * While the VectorFieldTextures array is split into flat textures, we need a way to 
 * sample a texture by index, this function wraps 
 * 
 * 	// @todo compat hack - remove this function
 */
 float3 SampleVectorFieldTexture(int Index, float3 UV)
 {
#if MAX_VECTOR_FIELDS == 1
	 return Texture3DSample(VectorFieldTextures_0, VectorFieldTexturesSampler_0, UV).xyz;
#else
	if (Index == 0) return Texture3DSample(VectorFieldTextures_0, VectorFieldTexturesSampler_0, UV).xyz;
	if (Index == 1) return Texture3DSample(VectorFieldTextures_1, VectorFieldTexturesSampler_1, UV).xyz;
	if (Index == 2) return Texture3DSample(VectorFieldTextures_2, VectorFieldTexturesSampler_2, UV).xyz;
	return Texture3DSample(VectorFieldTextures_3, VectorFieldTexturesSampler_3, UV).xyz;
#endif
 }

/**
 * Compute the influence of vector fields on a particle at the given position.
 * @param OutForce - Force to apply to the particle.
 * @param OutVelocity - Direct velocity influence on the particle.
 * @param Position - Position of the particle.
 * @param PerParticleScale - Amount by which to scale the influence on this particle.
 */
void EvaluateVectorFields(out float3 OutForce, out float4 OutVelocity, FLWCVector3 LWCPosition, float PerParticleScale)
{
	float3 TotalForce = 0;
	float3 WeightedVelocity = 0;
	float TotalWeight = 0;
	float FinalWeight = 0;

	for (int VectorFieldIndex = 0; VectorFieldIndex < VectorFields.Count; ++VectorFieldIndex)
	{
		FLWCVector3 WorldToVolumeTile = MakeLWCVector3(VectorFields.WorldToVolumeTile[VectorFieldIndex].xyz, 0.0f);
		float3 Position = LWCToFloat(LWCSubtract(LWCPosition, WorldToVolumeTile));

		float2 IntensityAndTightness = VectorFields.IntensityAndTightness[VectorFieldIndex].xy;
		float Intensity = IntensityAndTightness.x * PerParticleScale;
		float Tightness = IntensityAndTightness.y;
		float3 VolumeSize = VectorFields.VolumeSize[VectorFieldIndex].xyz;
		float3 VolumeUV = mul(float4(Position, 1.0f), VectorFields.WorldToVolume[VectorFieldIndex]).xyz;
		//Tile the UVs if needed. TilingAxes will be 1.0 or 0.0 in each channel depending on which axes are being tiled, if any.
		VolumeUV -= floor(VolumeUV * VectorFields.TilingAxes[VectorFieldIndex].xyz);

		float3 AxisWeights = 
			saturate(VolumeUV * VolumeSize.xyz) *
			saturate((1.0f - VolumeUV) * VolumeSize.xyz);
		float DistanceWeight = min(AxisWeights.x, min(AxisWeights.y, AxisWeights.z));

		// @todo compat hack: Some compilers only allow constant indexing into a texture array
//		float3 VectorSample = Texture3DSample(VectorFieldTextures[VectorFieldIndex], VectorFieldTexturesSampler, saturate(VolumeUV)).xyz;
		float3 VectorSample = SampleVectorFieldTexture(VectorFieldIndex, saturate(VolumeUV));

		float3 Vec = mul(float4(VectorSample,0), VectorFields.VolumeToWorld[VectorFieldIndex]).xyz;
		TotalForce += (Vec * DistanceWeight * Intensity);
		WeightedVelocity += (Vec * Intensity * DistanceWeight * Tightness);
		TotalWeight += (DistanceWeight * Tightness);
		FinalWeight = max(FinalWeight, DistanceWeight * Tightness);
	}

	// Forces are additive.
	OutForce = TotalForce;
	// Velocities use a weighted average.
	OutVelocity.xyz = WeightedVelocity / (TotalWeight + 0.001f);
	OutVelocity.w = FinalWeight;
}

/**
 * Compute the force due to drag.
 * @param Velocity - Velocity of the particle.
 * @param DragCoefficient - Coefficient of drag to apply to the particle.
 */
float3 ComputeDrag(float3 Velocity, float DragCoefficient)
{
	return -DragCoefficient * Velocity;
}

/**
 * Compute the force on the particle due to a point of attraction.
 * @param Position - The position of the particle.
 */
float3 ComputeAttractionForce(float3 Position)
{
	float3 PointLoc = PointAttractor.xyz;
	float RadiusSq = PointAttractor.w;
	float Strength = PositionOffsetAndAttractorStrength.w;

	float3 DirectionToPoint = PointLoc - Position + float3(0, 0, 0.0001f);
	float DistSq = max(dot(DirectionToPoint,DirectionToPoint), RadiusSq);
	float Attraction = Strength / DistSq;
	return Attraction * normalize(DirectionToPoint);
}

/** For retrieving the size of a particle. */
Texture2D RenderAttributesTexture;
SamplerState RenderAttributesTextureSampler;

#if DEPTH_BUFFER_COLLISION

/** Limits the depth bounds for which to search for a collision plane. */
float CollisionDepthBounds;

/** TODO: Should be moved to Common.usf and used globaly. */
float3 TranslatedWorldPositionFromSceneDepth(float2 ScreenPosition, float SceneDepth)
{
	float3 WorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	return WorldPosition;
}

float3 ComputeCollidingVelocity(
	in float3 MidVelocity,
	in float3 CollisionNormal,
	in float RelativeTime,
	in float Resilience)
{
	float3 PerpVelocity = dot(MidVelocity, CollisionNormal) * CollisionNormal;
	float3 TanVelocity = MidVelocity - PerpVelocity;
	float3 Reflection = Simulation.OneMinusFriction * TanVelocity - Resilience * PerpVelocity;

	if (Simulation.CollisionRandomSpread == 0)
	{
		return Reflection;
	}

	float Rand0 = frac(RelativeTime * 131) * 2 * 3.1415;
	float Rand1 = frac(RelativeTime * 937);
	
	float3 ReflectionLength = length(Reflection);
	Reflection *= 1.0 / ReflectionLength;

	// Spread particules uniformely through the reflection cone for View.GeneralPurposeTweak < 1.
	float RoN = dot(Reflection, CollisionNormal);
	float ZDistributionRange = 1 - Simulation.CollisionRandomSpread * pow(Rand1, Simulation.CollisionRandomDistribution) * (1 - sqrt(1 - RoN * RoN));
	float TangentDistributionRange = sqrt(1 - ZDistributionRange * ZDistributionRange);
	
	float3 UpVector = abs(Reflection.z) < 0.999 ? float3(0,0,1) : float3(1,0,0);
	float3 ReflectionTangentX = normalize( cross( UpVector, Reflection ) );
	float3 ReflectionTangentY = cross( Reflection, ReflectionTangentX );

	return ReflectionLength * (
		ReflectionTangentX * (cos(Rand0) * TangentDistributionRange) +
		ReflectionTangentY * (sin(Rand0) * TangentDistributionRange) +
		Reflection * ZDistributionRange);
}

/**
 * Compute collision with the depth buffer.
 */
void CollideWithDepthBuffer(
	out FLWCVector3 NewPosition_LWC,
	out float3 NewVelocity,
	inout float RelativeTime,
	in FLWCVector3 InPosition_LWC,
	in float3 InVelocity,
	in float3 Acceleration,
	in float CollisionRadius,
	in float Resilience
	)
{
	float3 InPosition = LWCToFloat(LWCAdd(InPosition_LWC, PrimaryView.TileOffset.PreViewTranslation));

	// Integration assuming no collision.
	float3 MidVelocity = InVelocity.xyz + 0.5f * Acceleration;
	float3 DeltaPosition = DeltaSeconds * MidVelocity;
	float3 NewPosition = InPosition.xyz + DeltaPosition;
	NewVelocity = InVelocity.xyz + Acceleration;

	// Figure out where to sample the depth buffer.
	float3 CollisionOffset = normalize(DeltaPosition) * CollisionRadius;

	float3 CollisionPosition = InPosition + CollisionOffset;
	float4 SamplePosition = float4(CollisionPosition, 1);
	float4 ClipPosition = mul(SamplePosition, View.TranslatedWorldToClip);
	float2 ScreenPosition = ClipPosition.xy / ClipPosition.w;

	// Don't try to collide if the particle falls outside the view.
	if (all(abs(ScreenPosition.xy) <= float2(1,1)))
	{
		// Sample the depth buffer to get a world position near the particle.
		float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
		float SceneDepth = CalcSceneDepth(ScreenUV);
		if (abs(GetScreenPositionDepth(ClipPosition) - SceneDepth) < CollisionDepthBounds)
		{
#if FORWARD_SHADING_ACCURATE_NORMAL && FORWARD_SHADING_NORMAL_CALC
			float SceneDepth0 = CalcSceneDepth(ScreenUV + float2(View.BufferSizeAndInvSize.z, 0.0));
			float SceneDepth1 = CalcSceneDepth(ScreenUV + float2(0.0, View.BufferSizeAndInvSize.w));
			// When using the forward shading, the normal of the pixel is approximated by the derivative of the world position
			// of the pixel. But in on the visible edge this derivative can become very high, making CollisionPlane almost
			// perpendicular to the view plane. In these case the particle may collide the visible edges of the diferent meshes
			// in the view frustum. To avoid this, we disable the collision test if one of the derivate is above a threshold.
			if (max(abs(SceneDepth - SceneDepth0), abs(SceneDepth - SceneDepth1)) > CollisionDepthBounds)
			{
				NewPosition_LWC = LWCSubtract(NewPosition, PrimaryView.TileOffset.PreViewTranslation);
				return;
			}
#endif
			// Reconstruct world position.
			float3 WorldPosition = TranslatedWorldPositionFromSceneDepth(ScreenPosition.xy, SceneDepth);
			
#if FORWARD_SHADING_NORMAL_CALC && FORWARD_SHADING_ACCURATE_NORMAL
			// Compute normal using the two additional neighbooring pixel depth buffer fetches.
			float3 WorldPosition0 = TranslatedWorldPositionFromSceneDepth(ScreenPosition.xy + float2(2 * View.ViewSizeAndInvSize.z, 0.0), SceneDepth0);
			float3 WorldPosition1 = TranslatedWorldPositionFromSceneDepth(ScreenPosition.xy - float2(0.0, 2 * View.ViewSizeAndInvSize.w), SceneDepth1);
			float3 WorldNormal = normalize(cross(WorldPosition0 - WorldPosition, WorldPosition1 - WorldPosition));
#elif FORWARD_SHADING_NORMAL_CALC // && !FORWARD_SHADING_ACCURATE_NORMAL
			// Compute normal using a ddx/ddy hack, hopefully the neighbooring particules will be close enough...
			float3 WorldNormal = normalize(cross(ddx(WorldPosition), ddy(WorldPosition)));
			WorldNormal *= sign(dot(ResolvedView.TranslatedWorldCameraOrigin - NewPosition, WorldNormal));
#else //!FORWARD_SHADING_NORMAL_CALC
			// Sample the normal buffer to create a plane to collide against.
			float3 WorldNormal = Texture2DSampleLevel(SceneTexturesStruct.GBufferATexture, SceneTexturesStruct_GBufferATextureSampler, ScreenUV, 0).xyz * 2.0 - 1.0;
#endif
			float4 CollisionPlane = float4(WorldNormal, dot(WorldPosition.xyz,WorldNormal));

			// Compute the portion of velocity normal to the collision plane.
			float VelocityDot = dot(CollisionPlane.xyz, DeltaPosition.xyz);

#if IMPROVED_DEPTH_BUFFER_COLLISION
			// distance to the plane from current and predicted position
			float d_back = ( dot(CollisionPlane.xyz, InPosition.xyz)+CollisionRadius - CollisionPlane.w );
			float d_front = ( dot(CollisionPlane.xyz, NewPosition.xyz)-CollisionRadius - CollisionPlane.w );

			if (d_back >= 0.0f && d_front <= 0.0f && VelocityDot<0.0f)
			{
				NewVelocity = ComputeCollidingVelocity(MidVelocity, CollisionPlane.xyz, RelativeTime, Resilience);

				// If the particle lies approximately on the collision plane, don't jump to the point of collision.
				d_back *= step(VelocityDot,-1);

				// Integrate position taking the collision in to account.
				float PositionAdjustment = ( dot(CollisionPlane.xyz, InPosition.xyz)-CollisionRadius - CollisionPlane.w );
				NewPosition = InPosition + PositionAdjustment*CollisionPlane.xyz + NewVelocity * DeltaSeconds*0.1;

				// Update the relative time. Usually this does nothing, but if the
				// user has elected to kill the particle upon collision this will do
				// so.
				RelativeTime += Simulation.CollisionTimeBias;
			}
			else if (d_front < 0.0f && d_back < 0.0f)
			{
				RelativeTime = 1.1f;
			}
#else
			float InvVelocityDot = rcp(VelocityDot + 0.0001f); // Add a small amount to avoid division by zero.
			// Distance to the plane from the center of the particle.
			float DistanceToPlane = dot(CollisionPlane.xyz, InPosition.xyz) - CollisionPlane.w;

			// Find out the time of intersection for both the front and back of the sphere.
			float t_back = -(DistanceToPlane + CollisionRadius) * InvVelocityDot;
			float t_front = -(DistanceToPlane - CollisionRadius) * InvVelocityDot;

			if (step(0, t_back) * step(t_front, 1) * step(0, DistanceToPlane))
			{
				NewVelocity = ComputeCollidingVelocity(MidVelocity, CollisionPlane.xyz, RelativeTime, Resilience);

				// If the particle lies approximately on the collision plane, don't jump to the point of collision.
				t_front *= step(VelocityDot,-1);

				// Integrate position taking the collision in to account.
				NewPosition = InPosition + DeltaPosition * t_front + NewVelocity * (1.0f - t_front) * DeltaSeconds;

				// Update the relative time. Usually this does nothing, but if the
				// user has elected to kill the particle upon collision this will do
				// so.
				RelativeTime += Simulation.CollisionTimeBias;
			}
			//else if (t_front > 0 && t_back < 1 && DistanceToPlane < 0)
			else if (step(0, t_front) * step(t_back, 1) * step(DistanceToPlane,0))
			{
				// The particle has collided against a backface, kill it by setting
				// relative time to a value > 1.0.
				RelativeTime = 1.1f;
			}
#endif
		}
	}

	NewPosition_LWC = LWCSubtract(NewPosition, PrimaryView.TileOffset.PreViewTranslation);
}
#endif // #if DEPTH_BUFFER_COLLISION

#if DISTANCE_FIELD_COLLISION

/**
 * Compute collision with the global signed distance field
 */
void CollideWithDistanceField(
	out FLWCVector3 NewPosition_LWC,
	out float3 NewVelocity,
	inout float RelativeTime,
	in FLWCVector3 InPosition_LWC,
	in float3 InVelocity,
	in float3 Acceleration,
	in float CollisionRadius,
	in float Resilience
	)
{
	float3 InPosition = LWCToFloat(LWCAdd(InPosition_LWC, PrimaryView.TileOffset.PreViewTranslation));

	// Integration assuming no collision.
	float3 MidVelocity = InVelocity.xyz + 0.5f * Acceleration;
	float3 DeltaPosition = DeltaSeconds * MidVelocity;
	float3 NewPosition = InPosition.xyz + DeltaPosition;
	NewVelocity = InVelocity.xyz + Acceleration;

	float DistanceToNearestSurface = GetDistanceToNearestSurfaceGlobal(InPosition);
	float MaxCollisionDistance = CollisionRadius + length(DeltaPosition.xyz);

	if (DistanceToNearestSurface < MaxCollisionDistance)
	{
		float3 CollisionWorldNormal = normalize(GetDistanceFieldGradientGlobal(InPosition));
		float3 CollisionWorldPosition = InPosition - CollisionWorldNormal * DistanceToNearestSurface;

		float4 CollisionPlane = float4(CollisionWorldNormal.xyz, dot(CollisionWorldPosition.xyz, CollisionWorldNormal.xyz));

		// Compute the portion of velocity normal to the collision plane.
		float VelocityDot = dot(CollisionPlane.xyz, DeltaPosition.xyz);
		float InvVelocityDot = rcp(VelocityDot + 0.0001f); // Add a small amount to avoid division by zero.

		// Distance to the plane from the center of the particle.
		float DistanceToPlane = dot(CollisionPlane.xyz, InPosition.xyz) - CollisionPlane.w;

		// Find out the time of intersection for both the front and back of the sphere.
		float t_back = -(DistanceToPlane + CollisionRadius) * InvVelocityDot;
		float t_front = -(DistanceToPlane - CollisionRadius) * InvVelocityDot;

		//if (t_back >= 0 && t_front <= 1 && DistanceToPlane >= 0)
		if (step(0, t_back) * step(t_front, 1) * step(0, DistanceToPlane))
		{
			// Separate velocity in to the components perpendicular and tangent to the collision plane.
			float3 PerpVelocity = dot(MidVelocity,CollisionPlane.xyz) * CollisionPlane.xyz;
			float3 TanVelocity = MidVelocity - PerpVelocity;

			// Compute the new velocity accounting for resilience and friction.
			NewVelocity = Simulation.OneMinusFriction * TanVelocity - Resilience * PerpVelocity;

			// If the particle lies approximately on the collision plane, don't jump to the point of collision.
			t_front *= step(VelocityDot,-1);

			// Integrate position taking the collision in to account.
			NewPosition = InPosition + DeltaPosition * t_front + NewVelocity * (1.0f - t_front) * DeltaSeconds;

			// Update the relative time. Usually this does nothing, but if the
			// user has elected to kill the particle upon collision this will do
			// so.
			RelativeTime += Simulation.CollisionTimeBias;
		}
		//else if (t_front > 0 && t_back < 1 && DistanceToPlane < 0)
		else if (step(0, t_front) * step(t_back, 1) * step(DistanceToPlane,0))
		{
			// The particle has collided against a backface, kill it by setting
			// relative time to a value > 1.0.
			RelativeTime = 1.1f;
		}
	}

	NewPosition_LWC = LWCSubtract(NewPosition, PrimaryView.TileOffset.PreViewTranslation);
}

#endif

void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_Target0,
	out float4 OutVelocity : SV_Target1
	)
{
	// Initialize force to the constant acceleration.
	float3 Force = Simulation.Acceleration;

	// Sample the current position, velocity, and attributes for this particle.
	const float4 PositionSample = Texture2DSample(PositionTexture, PositionTextureSampler, Interpolants.TexCoord.xy);
	const float4 VelocitySample = Texture2DSample(VelocityTexture, VelocityTextureSampler, Interpolants.TexCoord.xy );
	const float4 InitialAttributes = Texture2DSample(AttributesTexture, AttributesTextureSampler, Interpolants.TexCoord.xy ) *
		Simulation.AttributeScale + Simulation.AttributeBias;

	// Velocity.w holds the time scale for this particle.
	float3 Velocity = VelocitySample.xyz;
	const float TimeScale = VelocitySample.w;

	// Position.w holds the relative time of the particle.
	FLWCVector3 Position = MakeLWCVector3(LWCTile, PositionSample.xyz);
	float RelativeTime = PositionSample.w;

	for (int IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
	{
		RelativeTime += DeltaSeconds * TimeScale;

		// Sample the attribute curve.
		const float2 AttributeCurveTexCoord = Simulation.AttributeCurve.xy +
			Simulation.AttributeCurve.zw * RelativeTime;
		const float4 AttributeCurve = Texture2DSample(CurveTexture, CurveTextureSampler, AttributeCurveTexCoord ) *
			Simulation.AttributeCurveScale + Simulation.AttributeCurveBias;

		// Simulation attributes.
		const float4 Attributes = InitialAttributes * AttributeCurve;
		const float DragCoefficient = Attributes.r;
		const float PerParticleVectorFieldScale = Attributes.g;
		const float Resilience = Attributes.b;
		const float OrbitRandom = Attributes.a;

		// Evalute vector fields.
		float3 FieldForce = 0;
		float4 FieldVelocity = 0;
		EvaluateVectorFields(FieldForce, FieldVelocity, Position, PerParticleVectorFieldScale);

		// Add in force from vector fields.
		Force += FieldForce;

		// Account for direct velocity.
		const float DirectVelocityAmount = FieldVelocity.w;
		Velocity.xyz = lerp(Velocity.xyz, FieldVelocity.xyz, DirectVelocityAmount);

		// Compute force due to drag.
		Force += ComputeDrag(Velocity.xyz, DragCoefficient);

		// Compute force to a point gravity source.
		Force += ComputeAttractionForce(PositionSample.xyz);

		// Compute the acceleration to apply to the particle this frame.
		float3 Acceleration = Force * DeltaSeconds;

	#if DEPTH_BUFFER_COLLISION || DISTANCE_FIELD_COLLISION
		// We need to look up render attributes for this particle to figure out how big it is.
		float4 RenderAttributeSample = Texture2DSampleLevel(RenderAttributesTexture, RenderAttributesTextureSampler, Interpolants.TexCoord.xy, 0);
	
		// Sample the misc render attributes curve.
		float2 MiscCurveTexCoord = Simulation.MiscCurve.xy + Simulation.MiscCurve.zw * RelativeTime;
		float4 MiscCurveSample = Texture2DSampleLevel(CurveTexture, CurveTextureSampler, MiscCurveTexCoord, 0 );
		float4 MiscCurve = MiscCurveSample * Simulation.MiscScale + Simulation.MiscBias;

		// Compute the size of the sprite. Note it is (0,0) if the sprite is dead.
		float2 InitialSize = abs(RenderAttributeSample.xy);
		float2 SizeScale = MiscCurve.xy;
		float2 Size = InitialSize * SizeScale * LocalToWorldScale;

		// Compute the radius with which to perform collision checks.
		float CollisionRadius = min(Size.x,Size.y) * Simulation.CollisionRadiusScale + Simulation.CollisionRadiusBias;
	#endif

		FLWCVector3 NewPosition;
		float3 NewVelocity;

	#if DEPTH_BUFFER_COLLISION
		// Compute the new position and velocity of the particle by colliding against
		// the scene's depth buffer.
		CollideWithDepthBuffer(
			NewPosition,
			NewVelocity,
			RelativeTime,
			Position,
			Velocity,
			Acceleration,
			CollisionRadius,
			Resilience
		);
	#elif DISTANCE_FIELD_COLLISION
		CollideWithDistanceField(
			NewPosition,
			NewVelocity,
			RelativeTime,
			Position,
			Velocity,
			Acceleration,
			CollisionRadius,
			Resilience
			);
	#else 
		// Integrate position and velocity forward.
		float3 DeltaPosition = DeltaSeconds * (Velocity.xyz + 0.5f * Acceleration);
		NewPosition = LWCAdd(Position, DeltaPosition);
		NewVelocity = Velocity.xyz + Acceleration;
	#endif

		// Apply orbit.
		const float3 OrbitVelocity = ComputeOrbitVelocity(RelativeTime, OrbitRandom);
		NewPosition = LWCAdd(NewPosition, OrbitVelocity * DeltaSeconds);

		// Update values for new iteration.
		Velocity = NewVelocity;
		Position = NewPosition;
	}

	// Store the new position, time, and velocity for the particle.
	OutPosition.xyz = LWCToFloat(LWCSubtract(Position, MakeLWCVector3(LWCTile, 0))) + PositionOffsetAndAttractorStrength.xyz;
	OutPosition.w = RelativeTime;
	OutVelocity.xyz = Velocity;
	OutVelocity.w = TimeScale;
}

#endif // #if PARTICLE_SIMULATION_PIXELSHADER

/*------------------------------------------------------------------------------
	Clear particle simulation pixel shader.
------------------------------------------------------------------------------*/
#if PARTICLE_CLEAR_PIXELSHADER

void PixelMain(
	in FShaderInterpolants Interpolants,
	out float4 OutPosition : SV_Target0,
	out float4 OutVelocity : SV_Target1
	)
{
	// Relative time just needs to be >1.0f so the particle is considered dead.
	OutPosition = float4(0,0,0,2.0f);
	OutVelocity = float4(0,0,0,0);
}
 
#endif // #if PARTICLE_CLEAR_PIXELSHADER

