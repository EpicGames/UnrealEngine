// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	ParticleSortKeyGen.usf: Shader to generate particle sorting keys.
==============================================================================*/

/*------------------------------------------------------------------------------
	Compile time parameters:
		THREAD_COUNT - The number of threads to launch per workgroup.
		TEXTURE_SIZE_X - The width of the position texture.
		TEXTURE_SIZE_Y - The height of the position texture.
------------------------------------------------------------------------------*/

#include "Common.ush"

/** Input buffer containing particle indices. */
Buffer<float2> InParticleIndices;
/** Texture containing particle positions. */
Texture2D PositionTexture;

/** Output key buffer. */
RWBuffer<uint> OutKeys;
/** Output indices buffer. */
RWBuffer<float2> OutParticleIndices;

[numthreads(THREAD_COUNT,1,1)]
void GenerateParticleSortKeys(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupIdXYZ : SV_GroupID )
{
	// Determine group and thread IDs.
	const uint ThreadId = GroupThreadId.x;
	const uint GroupId = GroupIdXYZ.x;

	// Setup chunk indices.
	uint FirstChunkIndex;
	uint ChunkCount;
	if ( GroupId < ParticleKeyGen.ExtraChunkCount )
	{
		FirstChunkIndex = GroupId * (ParticleKeyGen.ChunksPerGroup + 1);
		ChunkCount = ParticleKeyGen.ChunksPerGroup + 1;
	}
	else
	{
		FirstChunkIndex = GroupId * ParticleKeyGen.ChunksPerGroup + ParticleKeyGen.ExtraChunkCount;
		ChunkCount = ParticleKeyGen.ChunksPerGroup;
	}

	// Buffer offsets.
	uint InputIndex = FirstChunkIndex * THREAD_COUNT + ThreadId;
	uint OutputIndex = ParticleKeyGen.OutputOffset + InputIndex;

	for ( uint ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex )
	{
		if ( InputIndex < ParticleKeyGen.KeyCount )
		{
			// Read in the particle index and its position.
			const float2 ParticleIndex = InParticleIndices[InputIndex];
			int3 ParticleTexel = int3(ParticleIndex.xy * int2(TEXTURE_SIZE_X, TEXTURE_SIZE_Y), 0);
			const float4 ParticlePosition = PositionTexture.Load(ParticleTexel);

			// Compute distance to the view.
			const float DistanceToView = length( ParticlePosition.xyz - ParticleKeyGen.ViewOrigin.xyz );
			//const float DistanceToView = ParticlePosition.x;

			// Compute the sorting key.
			uint Key = ~f32tof16( DistanceToView );
			Key = (Key & 0xffff) | ParticleKeyGen.EmitterKey;

			// Store the key and index.
			OutKeys[OutputIndex] = Key;
			OutParticleIndices[OutputIndex] = ParticleIndex;
		}
		InputIndex += THREAD_COUNT;
		OutputIndex += THREAD_COUNT;
	}
}
