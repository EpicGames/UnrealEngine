// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleSpriteVertexFactory.hlsl: Particle vertex factory shader code.
	Shared by standard sprite particles and SubUV sprite particles.
=============================================================================*/

#include "VertexFactoryCommon.ush"
#include "ParticleVertexFactoryCommon.ush"

#define USE_PARTICLE_LIGHTING_OFFSET (FEATURE_LEVEL >= FEATURE_LEVEL_SM5 && !MATERIAL_SHADINGMODEL_UNLIT && MATERIAL_DOMAIN_SURFACE)
#define USE_PARTICLE_POSITION (NEEDS_PARTICLE_POSITION || PASS_NEEDS_PRIMITIVE_VOLUME_BOUNDS)
#define USE_PARTICLE_VELOCITY (NEEDS_PARTICLE_VELOCITY)
#define USE_PARTICLE_TIME (NEEDS_PARTICLE_TIME)
#define USE_PARTICLE_SIZE (NEEDS_PARTICLE_SIZE)
#define USE_PARTICLE_SPRITE_ROTATION (NEEDS_PARTICLE_SPRITE_ROTATION)

// We don't compute a tangent basis in ES3.1 to save shader instructions.
#define NEEDS_TANGENT_BASIS (FEATURE_LEVEL >= FEATURE_LEVEL_SM4)

struct FVertexFactoryInput
{
	float4	Position				: ATTRIBUTE0;
	float4	OldPosition				: ATTRIBUTE1;
	float4	SizeRotSubImage			: ATTRIBUTE2;
	float4	Color					: ATTRIBUTE3;
#if USE_DYNAMIC_PARAMETERS
	float4 DynamicParameter			: ATTRIBUTE5;
#endif	//USE_DYNAMIC_PARAMETERS
	float2	TexCoord				: ATTRIBUTE4;
	uint VertexId					: SV_VertexID;

	VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

struct FVertexFactoryInterpolantsVSToPS
{
#if NEEDS_TANGENT_BASIS
	// First row of the tangent to world matrix, Interp_Sizer used by SUBUV_PARTICLES in w
	float4	TangentToWorld0AndInterp_Sizer : TANGENTTOWORLD0;
	// Last row of the tangent to world matrix in xyz
	float4	TangentToWorld2	: TANGENTTOWORLD2;
#else
	float SubImageLerp : TEXCOORD0;
#endif

#if USE_DYNAMIC_PARAMETERS
	nointerpolation float4 DynamicParameter	: TEXCOORD1;
#endif	//USE_DYNAMIC_PARAMETERS

#if NEEDS_PARTICLE_COLOR
	float4	Color				: TEXCOORD2;
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	float4	TexCoords[(NUM_TEX_COORD_INTERPOLATORS + 1) / 2]	: TEXCOORD3;
#endif

	//Not sure this is actually being used now and it's awkward to slot in now we're supporting custom UVs so I'm just giving this its own interpolant.
#if LIGHTMAP_UV_ACCESS
	float2 LightMapUVs : LIGHTMAP_UVS;
#endif

#if USE_PARTICLE_SUBUVS
	float4 ParticleSubUVs : PARTICLE_SUBUVS;
#endif

#if USE_PARTICLE_POSITION
	/** Cam-relative (translated) particle center and radius */
	float4 ParticleTranslatedWorldPositionAndSize	: PARTICLE_POSITION;
	#if GENERATE_SPHERICAL_PARTICLE_NORMALS
		float3 DisplacedParticleCenterForNormal : PARTICLE_POSITION_FSN;
	#endif
#endif

#if USE_PARTICLE_VELOCITY
	float4 ParticleVelocity : PARTICLE_VELOCITY;
#endif

#if USE_PARTICLE_TIME
	float ParticleTime : PARTICLE_TIME;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset : PARTICLE_LIGHTING_OFFSET;
#endif

#if USE_PARTICLE_SIZE
	float2 ParticleSize : PARTICLE_SIZE;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	float ParticleSpriteRotation : PARTICLE_SPRITE_ROTATION;
#endif
};

struct FVertexFactoryIntermediates
{
	/** The position of the particle in translated world space. */
	float3 ParticleTranslatedWorldPosition;
	/** The position of the vertex in translated world space. */
	float3 VertexWorldPosition;
	/** Particle translated world space position and size. */
	float4 TranslatedWorldPositionAndSize;
	
#if USE_PARTICLE_LIGHTING_OFFSET
	float3 LightingPositionOffset;
#endif

	/** The texture coordinate at this vertex. */
	float4 TexCoord;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3 //Unflipped UVs are placed in UVs 2-3
	/** A second UV set. Always non-UV flipped. Allows use of UV flipping with Normal maps etc. */
	float4 TexCoord_Unflipped;
#endif
	/** The sprite tangent in world space (+V). */
	float3 TangentUp;
	/** The sprite tangent in world space (+U). */
	float3 TangentRight;
	/** The color of the sprite. */
	float4 Color;
	/** The delta between previous and current particle locations. */
	float3 ParticlePrevToCurrDelta;
	/** The velocity of the particle, XYZ: direction, W: speed. */
	float4 ParticleVelocity;
	/** Dynamic parameter. */
	float4 DynamicParameter;
	/** The sub-image lerp. */
	float SubImageLerp;
	/** Relative time. */
	float RelativeTime;
	/** Transform from tangent space to local space. */
	float3x3 TangentToLocal;
	/** Size of the particle */
	float2 ParticleSize;
#if USE_PARTICLE_SPRITE_ROTATION
	float ParticleSpriteRotation : PARTICLE_SPRITE_ROTATION;
#endif
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

#if NUM_TEX_COORD_INTERPOLATORS
bool UVIndexUseZW(int UVIndex)
{
	return (UVIndex % 2) != 0;
}

float2 GetUV(FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return UVIndexUseZW(UVIndex) ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FVertexFactoryInterpolantsVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (UVIndexUseZW(UVIndex))
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

/** Converts from vertex factory specific interpolants to a FMaterialPixelParameters, which is used by material inputs. */
FMaterialPixelParameters GetMaterialPixelParameters(FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition)
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = GetUV(Interpolants, CoordinateIndex);
	}
#endif

	Result.VertexColor = 1;

#if NEEDS_TANGENT_BASIS
	half4 TangentToWorld0 = Interpolants.TangentToWorld0AndInterp_Sizer;
	half4 TangentToWorld2 = Interpolants.TangentToWorld2;
	float SubImageLerp = Interpolants.TangentToWorld0AndInterp_Sizer.w;

	#if GENERATE_SPHERICAL_PARTICLE_NORMALS && USE_PARTICLE_POSITION
		{	
			// Use displaced position (SvPosition already includes WPO)
			float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
			float3 SphereCenter = Interpolants.DisplacedParticleCenterForNormal;
			Result.TangentToWorld = GetSphericalParticleNormal(TranslatedWorldPosition, SphereCenter, Interpolants.ParticleTranslatedWorldPositionAndSize.w);
		}
	#else
		Result.TangentToWorld = AssembleTangentToWorld(TangentToWorld0.xyz, TangentToWorld2);
	#endif
#else
	float SubImageLerp = Interpolants.SubImageLerp;
#endif

	Result.UnMirrored = 1;
	Result.Particle.MacroUV = SpriteVF.MacroUVParameters;

#if NEEDS_PARTICLE_COLOR
	Result.Particle.Color = Interpolants.Color;
#endif

#if USE_DYNAMIC_PARAMETERS
	Result.Particle.DynamicParameter = Interpolants.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Interpolants.ParticleTranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Interpolants.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Interpolants.ParticleTime;
#endif

	Result.Particle.MotionBlurFade = 1.0f;


#if USE_PARTICLE_LIGHTING_OFFSET
	Result.LightingPositionOffset = Interpolants.LightingPositionOffset;
#endif

#if USE_PARTICLE_SUBUVS
	Result.Particle.SubUVCoords[0] = Interpolants.ParticleSubUVs.xy;
	Result.Particle.SubUVCoords[1] = Interpolants.ParticleSubUVs.zw;
	Result.Particle.SubUVLerp = SubImageLerp;
#endif

#if LIGHTMAP_UV_ACCESS
	Result.LightmapUVs = Interpolants.LightMapUVs;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Interpolants.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Interpolants.ParticleSpriteRotation;
#endif

	Result.TwoSidedSign = 1;
	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/** Converts from vertex factory specific input to a FMaterialVertexParameters, which is used by vertex shader material inputs. */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = Input.Color;
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3()); 
	Result.Particle.MacroUV = SpriteVF.MacroUVParameters;
	Result.Particle.Color = Intermediates.Color;
	Result.Particle.MotionBlurFade = 1.0f;
	Result.PreSkinnedPosition = Input.Position.xyz;
	Result.PreSkinnedNormal = TangentToLocal[2].xyz;

	// Previous frame not handled deliberately. Lacks necessary information and
	// primitives using this VF are usually transparent and hence don't output velocity
	Result.PrevFrameLocalToWorld = GetPrimitiveDataFromUniformBuffer().LocalToWorld;

#if USE_DYNAMIC_PARAMETERS
	Result.Particle.DynamicParameter = Intermediates.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

#if USE_PARTICLE_POSITION
	Result.Particle.TranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
	Result.Particle.PrevTranslatedWorldPositionAndSize = Result.Particle.TranslatedWorldPositionAndSize;
#endif

#if USE_PARTICLE_VELOCITY
	Result.Particle.Velocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Result.Particle.RelativeTime = Intermediates.RelativeTime;
#endif

#if NUM_MATERIAL_TEXCOORDS_VERTEX
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 1
		Result.TexCoords[0] = Intermediates.TexCoord.xy;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 2
		Result.TexCoords[1] = Intermediates.TexCoord.zw;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
		Result.TexCoords[2] = Intermediates.TexCoord_Unflipped.xy;
	#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 4
		Result.TexCoords[3] = Intermediates.TexCoord_Unflipped.zw;
	#endif	// >= 4
	#endif	// >= 3
	#endif	// >= 2
	#endif	// >= 1
#endif	//NUM_MATERIAL_TEXCOORDS_VERTEX

#if USE_PARTICLE_SUBUVS
	Result.Particle.SubUVCoords[0] = Intermediates.TexCoord.xy;
	Result.Particle.SubUVCoords[1] = Intermediates.TexCoord.zw;
#endif

#if USE_PARTICLE_SIZE
	Result.Particle.Size = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Result.Particle.SpriteRotation = Intermediates.ParticleSpriteRotation;
#endif

	Result.LWCData = MakeMaterialLWCData(Result);

#if ENABLE_NEW_HLSL_GENERATOR
	// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
	EvaluateVertexMaterialAttributes(Result);
#endif

	return Result;
}

float3 SafeNormalize(float3 V)
{
	return V * rsqrt(max(dot(V,V),0.00000001));
}

void GetTangents(float3 TranslatedWorldPosition, float3 OldTranslatedWorldPosition, float SpriteRotation, out float3 OutRight, out float3 OutUp)
{
	// Select camera up/right vectors.
	float3 ResolvedViewRight = lerp(ResolvedView.ViewRight, ResolvedView.HMDViewNoRollRight, SpriteVF.RemoveHMDRoll);
	float3 ResolvedViewUp = lerp(ResolvedView.ViewUp, ResolvedView.HMDViewNoRollUp, SpriteVF.RemoveHMDRoll);

	float3 CameraRight = lerp(ResolvedViewRight, SpriteVF.AxisLockRight.xyz, SpriteVF.AxisLockRight.w);
	float3 CameraUp = lerp(-ResolvedViewUp, SpriteVF.AxisLockUp.xyz, SpriteVF.AxisLockUp.w);

	// Determine the vector from the particle to the camera and the particle's movement direction.
	float3 CameraDirection = -GetCameraVectorFromTranslatedWorldPosition(ResolvedView, TranslatedWorldPosition);
	float3 RightVector = CameraRight.xyz;
	float3 UpVector = CameraUp.xyz;

	float4 LocalTangentSelector = SpriteVF.TangentSelector;

	BRANCH
	if (SpriteVF.CameraFacingBlend.x > 0.f)
	{
		// Blend between PSA_FacingCamera and PSA_Square over distance
		float CameraDistanceSq = GetDistanceToCameraFromViewVectorSqr(ResolvedView.TranslatedWorldCameraOrigin - TranslatedWorldPosition);
		float AlignmentMode = saturate(CameraDistanceSq * SpriteVF.CameraFacingBlend.y - SpriteVF.CameraFacingBlend.z);

		float3 CameraFacingRight = SafeNormalize(cross(CameraDirection, float3(0,0,1)));
		float3 CameraFacingUp = cross(CameraDirection, CameraFacingRight);

		RightVector = normalize(lerp(RightVector, CameraFacingRight, AlignmentMode));
		UpVector = normalize(lerp(UpVector, CameraFacingUp, AlignmentMode));
	}
	else
	{
		FLATTEN
		if (LocalTangentSelector.y > 0)
		{
			// Tangent vectors for PSA_Velocity.
			float3 ParticleDirection = SafeNormalize(TranslatedWorldPosition - OldTranslatedWorldPosition);
			RightVector	= SafeNormalize(cross(CameraDirection, ParticleDirection));
			UpVector = -ParticleDirection;
		}
		else if (LocalTangentSelector.z > 0)
		{
			// Tangent vectors for rotation locked about an axis.
			RightVector = SpriteVF.AxisLockRight.xyz;
			UpVector = -SafeNormalize(cross(RightVector,CameraDirection));
		}
		else if (LocalTangentSelector.w > 0)
		{
			// Tangent vectors for camera facing position.
			RightVector = SafeNormalize(cross(CameraDirection,float3(0,0,1)));
			UpVector = cross(CameraDirection, RightVector);
		}
	}

	// Determine the angle of rotation.
	float SinRotation; // = 0
	float CosRotation; // = 1
	sincos(SpriteRotation, SinRotation, CosRotation);

	// Rotate the sprite to determine final tangents.
	OutRight	= SinRotation * UpVector + CosRotation * RightVector;
	OutUp		= CosRotation * UpVector - SinRotation * RightVector;
}

/** derive basis vectors */
float3x3 CalcTangentBasis(FVertexFactoryIntermediates Intermediates)
{
#if NEEDS_TANGENT_BASIS
	// Using camera facing TangentX and TangentY.  The resulting tangent basis is not orthonormal with anything other than ENM_CameraFacing, 
	// So there are artifacts with tangent space calculations like the TransformVector node,
	// But this allows lighting based on a world space shape via the normal while still having normal maps camera aligned.
	float3x3 Result;
	Result[0] = Intermediates.TangentRight;
	Result[1] = Intermediates.TangentUp;

	float3 ParticleWorldPosition = Intermediates.ParticleTranslatedWorldPosition + DFHackToFloat(ResolvedView.PreViewTranslation);

	// ENM_CameraFacing
	//@todo - use static branching
	if (SpriteVF.NormalsType < .5f)
	{
		Result[2] = normalize(cross(Result[0],Result[1]));
	} 
	// ENM_Spherical
	else if (SpriteVF.NormalsType < 1.5f)
	{
		float3 TangentZ = normalize(ParticleWorldPosition - SpriteVF.NormalsSphereCenter.xyz);
		Result[2] = TangentZ;	
	}
	// ENM_Cylindrical
	else
	{
		float3 ClosestPointOnCylinder = SpriteVF.NormalsSphereCenter.xyz + dot(SpriteVF.NormalsCylinderUnitDirection.xyz, ParticleWorldPosition - SpriteVF.NormalsSphereCenter.xyz) * SpriteVF.NormalsCylinderUnitDirection.xyz;
		float3 TangentZ = normalize(ParticleWorldPosition - ClosestPointOnCylinder);
		Result[2] = TangentZ;	
	}

	return Result;
#else
	// Return the identity matrix.
	return float3x3(1,0,0,0,1,0,0,0,1);
#endif
}

// Note: C++ geometry setup behavior has to match this define
#define SUPPORTS_PARTICLE_CUTOUTS (FEATURE_LEVEL >= FEATURE_LEVEL_ES3_1)
#if SUPPORTS_PARTICLE_CUTOUTS

/** Number of vertices in each SubUV frame of CutoutGeometry. */
uint NumCutoutVerticesPerFrame;

/** SubUV animation bounding geometry used to minimize overdraw.  Each frame has its own polygon. */
Buffer<float2> CutoutGeometry;

#endif

void ComputeBillboardUVs(FVertexFactoryInput Input, out float2 UVForPosition, out float2 UVForTexturing, out float2 UVForTexturingUnflipped)
{
	// Encoding the UV flip in the sign of the size data.
	float2 UVFlip = sign(Input.SizeRotSubImage.xy);

#if SUPPORTS_PARTICLE_CUTOUTS
	BRANCH
	if (NumCutoutVerticesPerFrame > 0)
	{
		// Avoid uint divide which is extremely slow on GCN
		uint CutoutVertexIndex = (uint)fmod(Input.VertexId, NumCutoutVerticesPerFrame);
		float NumFrames = SpriteVF.SubImageSize.x * SpriteVF.SubImageSize.y;
		uint SubImageIndexInt = (uint)fmod(Input.SizeRotSubImage.w, NumFrames);

		FLATTEN
		if (UVFlip.x * UVFlip.y < 0)
		{
			// Reverse the winding order of the polygon when only flipping in one direction to counteract UV flipping
			CutoutVertexIndex = NumCutoutVerticesPerFrame - 1 - CutoutVertexIndex;
		}

		// Fetch the billboard space positions from the appropriate frame of the preprocessed cutout geometry
		UVForTexturing = UVForTexturingUnflipped = CutoutGeometry[SubImageIndexInt * NumCutoutVerticesPerFrame + CutoutVertexIndex];

		// Invert positions on the billboard so that the cutout geometry still contains the visible portion
		// This changes the winding order when only one direction is flipped
		UVForPosition.x = UVFlip.x < 0.0 ? 1.0 - UVForTexturing.x : UVForTexturing.x;
		UVForPosition.y = UVFlip.y < 0.0 ? 1.0 - UVForTexturing.y : UVForTexturing.y;
	}
	else
#endif
	{
		UVForTexturing.x = UVFlip.x < 0.0 ? 1.0 - Input.TexCoord.x : Input.TexCoord.x;
		UVForTexturing.y = UVFlip.y < 0.0 ? 1.0 - Input.TexCoord.y : Input.TexCoord.y;

		// Note: not inverting positions, as that would change the winding order
		UVForPosition = UVForTexturingUnflipped = Input.TexCoord.xy;
	}
}


FVertexFactoryIntermediates GetVertexFactoryIntermediates(FVertexFactoryInput Input)
{
	FVertexFactoryIntermediates Intermediates = (FVertexFactoryIntermediates)0;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);

	// World position.
	FLWCVector3 LWCTileOffset = MakeLWCVector3(SpriteVF.LWCTile, 0);
	FLWCMatrix Transform = DFToTileOffset(GetPrimitiveDataFromUniformBuffer().LocalToWorld);
	FLWCVector3 ParticleWorldPosition = LWCAdd(LWCMultiply(Input.Position.xyz, Transform), LWCTileOffset);
	FLWCVector3 ParticleOldWorldPosition = LWCAdd(LWCMultiply(Input.OldPosition.xyz, Transform), LWCTileOffset);
	float3 ParticleTranslatedWorldPosition = LWCToFloat(LWCAdd(ParticleWorldPosition, ResolvedView.TileOffset.PreViewTranslation));
	float3 ParticleOldTranslatedWorldPosition = LWCToFloat(LWCAdd(ParticleOldWorldPosition, ResolvedView.TileOffset.PreViewTranslation));

	Intermediates.ParticleTranslatedWorldPosition = ParticleTranslatedWorldPosition;

	const float SpriteRotation = Input.SizeRotSubImage.z * SpriteVF.RotationScale + SpriteVF.RotationBias;

	// Tangents.
	float3 Right,Up;
	GetTangents(ParticleTranslatedWorldPosition, ParticleOldTranslatedWorldPosition, SpriteRotation, Right, Up);
	Intermediates.TangentUp = Up;
	Intermediates.TangentRight = Right;

	float2 UVForPosition;
	float2 UVForTexturing;
	float2 UVForTexturingUnflipped;
	ComputeBillboardUVs(Input, UVForPosition, UVForTexturing, UVForTexturingUnflipped);

	// Vertex position.
	float4 VertexWorldPosition = float4(ParticleTranslatedWorldPosition, 1);
	float2 Size = abs(Input.SizeRotSubImage.xy);
	VertexWorldPosition += Size.x * (UVForPosition.x + SpriteVF.PivotOffset.x) * float4(Right,0);
	VertexWorldPosition += Size.y * (UVForPosition.y + SpriteVF.PivotOffset.y) * float4(Up,0);
	Intermediates.VertexWorldPosition = VertexWorldPosition.xyz;
	
	// SubUV.
	float SubImageIndex = Input.SizeRotSubImage.w;
	float SubImageLerp = frac(SubImageIndex);

	// add a bias to SubImageA to avoid numerical precision issues around SubImageSize.x vs SubImageSize.z
	float SubImageA = SubImageIndex - SubImageLerp + 0.5f;
	float SubImageB = SubImageA + 1;
	float SubImageAH = floor( fmod( SubImageA, SpriteVF.SubImageSize.x ) );
	float SubImageBH = floor( fmod( SubImageB, SpriteVF.SubImageSize.x ) );
	float SubImageAV = floor( SubImageA * SpriteVF.SubImageSize.z );
	float SubImageBV = floor( SubImageB * SpriteVF.SubImageSize.z );
	Intermediates.TexCoord.xy = (float2( SubImageAH, SubImageAV ) + UVForTexturing) * SpriteVF.SubImageSize.zw ;
	Intermediates.TexCoord.zw = (float2(SubImageBH, SubImageBV) + UVForTexturing) * SpriteVF.SubImageSize.zw;
#if NUM_MATERIAL_TEXCOORDS_VERTEX >= 3
	Intermediates.TexCoord_Unflipped.xy = (float2(SubImageAH, SubImageAV) + UVForTexturingUnflipped) * SpriteVF.SubImageSize.zw;
	Intermediates.TexCoord_Unflipped.zw = (float2(SubImageBH, SubImageBV) + UVForTexturingUnflipped) * SpriteVF.SubImageSize.zw;
#endif
	Intermediates.SubImageLerp = SubImageLerp;

	Intermediates.Color = Input.Color;

#if USE_DYNAMIC_PARAMETERS
	Intermediates.DynamicParameter = Input.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS


#if USE_PARTICLE_POSITION
	float ParticleRadius = .5f * min(Input.SizeRotSubImage.x, Input.SizeRotSubImage.y);
	Intermediates.TranslatedWorldPositionAndSize = float4(ParticleTranslatedWorldPosition, ParticleRadius);

#if GENERATE_SPHERICAL_PARTICLE_NORMALS
	// Recenter the particle position for GENERATE_SPHERICAL_PARTICLE_NORMALS to use the right center.
	Intermediates.TranslatedWorldPositionAndSize.xyz += Size.x * (SpriteVF.PivotOffset.x + .5) * Right + Size.y * (SpriteVF.PivotOffset.y + .5) * Up;
#endif
#endif

#if !USE_PARTICLE_VELOCITY
	if (SpriteVF.UseVelocityForMotionBlur > 0.0f)
#endif
	{
		Intermediates.ParticlePrevToCurrDelta = ParticleTranslatedWorldPosition - ParticleOldTranslatedWorldPosition;
	#if USE_PARTICLE_VELOCITY
		float3 ParticleVelocity = Intermediates.ParticlePrevToCurrDelta * SpriteVF.InvDeltaSeconds;
		Intermediates.ParticleVelocity.xyz = normalize(ParticleVelocity);
		Intermediates.ParticleVelocity.w = length(ParticleVelocity);
	#endif
	}

#if USE_PARTICLE_TIME
	Intermediates.RelativeTime = Input.Position.w;
#endif

	Intermediates.TangentToLocal = CalcTangentBasis(Intermediates);

#if USE_PARTICLE_LIGHTING_OFFSET
	// Hash function based on the particle ID to generate a uniformly distributed 3d offset
	float3 RandomParticleOffset = frac(Square(Input.OldPosition.w + 10) * float3(1361.456345, 2333.578, 3623.983)) * 2 - 1;
	Intermediates.LightingPositionOffset = .5f * RandomParticleOffset;
#endif

#if USE_PARTICLE_SIZE
	Intermediates.ParticleSize = Size;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Intermediates.ParticleSpriteRotation = SpriteRotation;
#endif

	return Intermediates;
}

float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float4(Intermediates.VertexWorldPosition,1);
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return CalcTangentBasis(Intermediates)[2];
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 TranslatedWorldPosition)
{
#if SPHERICAL_PARTICLE_OPACITY
	// For particles using spherical opacity, move the quad toward the viewer so that it lies in front of the sphere defined by the particle
	// This avoids opaque objects intersecting the particle from causing clipping artifacts due to depth testing
	// The downside is that the particle will clip the near plane sooner

	float Radius = .5f * min(Input.SizeRotSubImage.x, Input.SizeRotSubImage.y);
	return ReprojectPosition(TranslatedWorldPosition, Radius);
#else
	return TranslatedWorldPosition;
#endif
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
#if SUPPORTS_PARTICLE_CUTOUTS && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL)
	// Do per-vertex lighting with particle position instead of vertex position when we're using SubUV cutouts, since the vertex positions vary per-frame which would cause popping
	return NumCutoutVerticesPerFrame > 0 ? Intermediates.ParticleTranslatedWorldPosition : TranslatedWorldPosition;
#else
	return TranslatedWorldPosition;
#endif
}

FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters)
{
	FVertexFactoryInterpolantsVSToPS Interpolants;

	// Initialize the whole struct to 0
	Interpolants = (FVertexFactoryInterpolantsVSToPS)0;

#if NUM_TEX_COORD_INTERPOLATORS
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
	{
		SetUV(Interpolants, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
	}
#endif

#if LIGHTMAP_UV_ACCESS
	Interpolants.LightMapUVs = Intermediates.TexCoord.xy;
#endif

#if USE_PARTICLE_SUBUVS
	Interpolants.ParticleSubUVs.xy = VertexParameters.Particle.SubUVCoords[0];
	Interpolants.ParticleSubUVs.zw = VertexParameters.Particle.SubUVCoords[1];
#endif

#if NEEDS_TANGENT_BASIS
	// Calculate the transform from tangent to world space.
	// Note that "local" space for particles is actually oriented in world space! Therefore no rotation is needed.
	float3x3 TangentToWorld = Intermediates.TangentToLocal;

	Interpolants.TangentToWorld0AndInterp_Sizer.xyz = TangentToWorld[0];
	Interpolants.TangentToWorld0AndInterp_Sizer.w = Intermediates.SubImageLerp;
	Interpolants.TangentToWorld2 = float4(TangentToWorld[2], sign(determinant(Intermediates.TangentToLocal)));
#else
	Interpolants.SubImageLerp = Intermediates.SubImageLerp;
#endif

#if NEEDS_PARTICLE_COLOR
	Interpolants.Color = Intermediates.Color;
#endif

#if USE_DYNAMIC_PARAMETERS
	Interpolants.DynamicParameter = Intermediates.DynamicParameter;
#endif	//USE_DYNAMIC_PARAMETERS

#if USE_PARTICLE_POSITION
	Interpolants.ParticleTranslatedWorldPositionAndSize = Intermediates.TranslatedWorldPositionAndSize;
	#if GENERATE_SPHERICAL_PARTICLE_NORMALS
		// Center for spherical normal must include WPO so shading follows displaced vertices
		Interpolants.DisplacedParticleCenterForNormal = Intermediates.TranslatedWorldPositionAndSize.xyz + GetMaterialWorldPositionOffset(VertexParameters);
	#endif
#endif

#if USE_PARTICLE_VELOCITY
	Interpolants.ParticleVelocity = Intermediates.ParticleVelocity;
#endif

#if USE_PARTICLE_TIME
	Interpolants.ParticleTime = Intermediates.RelativeTime;
#endif

#if USE_PARTICLE_LIGHTING_OFFSET
	Interpolants.LightingPositionOffset = Intermediates.LightingPositionOffset;
#endif

#if USE_PARTICLE_SIZE
	Interpolants.ParticleSize = Intermediates.ParticleSize;
#endif

#if USE_PARTICLE_SPRITE_ROTATION
	Interpolants.ParticleSpriteRotation = Intermediates.ParticleSpriteRotation;
#endif

	return Interpolants;
}

float4 VertexFactoryGetPreviousWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	float4 Position = VertexFactoryGetWorldPosition(Input, Intermediates);
	Position.xyz -= Intermediates.ParticlePrevToCurrDelta * SpriteVF.UseVelocityForMotionBlur;
	return Position;
}

// local position relative to primitive
float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Input.Position.xyz;
}

/**
* Get the 3x3 tangent basis vectors for this vertex factory
*
* @param Input - vertex input stream structure
* @return 3x3 matrix
*/
float3x3 VertexFactoryGetTangentToLocal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.TangentToLocal;
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
#if USE_PARTICLE_POSITION
	return Interpolants.ParticleTranslatedWorldPositionAndSize;
#endif
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#include "VertexFactoryDefaultInterface.ush"