// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ParticleVertexFactoryCommon.usf: 
=============================================================================*/

/** Generates a normal given the particle sphere, for a particle which always faces against the camera direction. */
float3x3 GetSphericalParticleNormal(float3 WorldPosition, float3 ParticlePosition, float ParticleRadius)
{
	//@todo - should be able to simplify this quite a bit
	float DistanceToParticle = length(WorldPosition - ParticlePosition);
	float Determinant = ParticleRadius * ParticleRadius - DistanceToParticle * DistanceToParticle;

	float3 WorldNormal = float3(0, 0, 1);

	FLATTEN
	if (Determinant > 0)
	{
		float SphereDistance = sqrt(Determinant);
		float3 PositionOnSphere = WorldPosition - ResolvedView.ViewForward * SphereDistance;
		WorldNormal = normalize(PositionOnSphere - ParticlePosition);
	}

	// Generate some continuous tangents based on the sphere normal
	// Constraining to the view up so X in the normal map tangent space will be to the right on the screen, y will be up on the screen
	float3 TangentX = normalize(cross(ResolvedView.ViewUp, WorldNormal));
	float3 TangentY = cross(WorldNormal, TangentX);
	return float3x3(TangentX, TangentY, WorldNormal);
}

float4 ReprojectPosition(float4 TranslatedWorldPosition, float DepthOffset)
{
#if SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS
	return TranslatedWorldPosition;
#else
	// Transform into view space
	float4 ViewSpacePosition = mul(TranslatedWorldPosition, ResolvedView.TranslatedWorldToView);
	float ViewDepth = ViewSpacePosition.z;
	// Project the vertex onto the near plane, then reposition at the new depth
	float4 ReprojectedViewSpacePosition = float4(ViewSpacePosition.xy * (ViewDepth - DepthOffset) / ViewDepth, ViewDepth - DepthOffset, 1);
	// Transform back into translated world space
	float4 ReprojectedWorldPosition = mul(ReprojectedViewSpacePosition, ResolvedView.ViewToTranslatedWorld);
	return ReprojectedWorldPosition;
#endif
}