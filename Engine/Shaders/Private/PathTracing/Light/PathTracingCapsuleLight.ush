// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingCapsuleLight.ush: Light sampling functions for capsule case of point lights
===============================================================================================*/

#pragma once

#include "PathTracingLightCommon.ush"
#include "../../CapsuleLightSampling.ush"

#define CAPSULE_SOLIDANGLE_SAMPLING		1

float3 CapsuleNormal(float3 Pos, float3 Center, float3 Axis)
{
	float3 pa = Pos - Center;
	float h = clamp(dot(pa, Axis) / dot(Axis, Axis), -0.5, 0.5);
	return normalize(pa - h * Axis);
}

float GetCapsuleRadius(int LightId, float3 Origin, float3 Axis, float Length)
{
    float h = clamp(dot(Axis, Origin), -0.5 * Length, 0.5 * Length);
	float3 ClosestPointOnAxis = Origin - Axis * h;
	float DistanceToAxisSqr = dot(ClosestPointOnAxis, ClosestPointOnAxis);

	// don't allow 0 area capsules since we need to divide the light power by area to get radiance
    // Also, shrink capsule if we are inside it
    return max(min(sqrt(0.99 * DistanceToAxisSqr), GetRadius(LightId)), 0.01);
}

FLightHit CapsuleLight_TraceLight(FRayDesc Ray, int LightId)
{
	float3 TranslatedLightPosition = GetTranslatedPosition(LightId);
    float3 Origin = TranslatedLightPosition - Ray.Origin;
    float3 Axis = GetdPdv(LightId);
    float SourceLength = GetSourceLength(LightId);
	float Radius = GetCapsuleRadius(LightId, Origin, Axis, SourceLength);
	float Radius2 = Pow2(Radius);

#if 0 // DEBUG code: visualize the bounding rectangle against the capsule intersection test
	FCapsuleSphericalBounds CapsuleBounds = CapsuleGetSphericalBounds(TranslatedLightPosition - Ray.Origin, GetdPdv(LightId), Radius, SourceLength);

	float ConeSolidAngle = CapsuleBounds.ConeSolidAngle;
	float RectSolidAngle = CapsuleBounds.SphericalRect.SolidAngle;
	bool HitBounds = false;
	float t = -1;
	if (ConeSolidAngle < RectSolidAngle)
	{
		// Is ray pointing inside the cone of directions?
		float CosTheta = dot(normalize(Ray.Direction), CapsuleBounds.ConeAxis);
		t = CosTheta >= SqrtOneMinusX(CapsuleBounds.ConeSinThetaMax2) ? length(TranslatedLightPosition) : -1.0;
	}
	else
	{
		float DoN = dot(Ray.Direction, CapsuleBounds.SphericalRect.Axis[2]);
		t = CapsuleBounds.SphericalRect.z0 / DoN;
		// ray points toward the plane and intersect it?
		if (t > Ray.TMin && t < Ray.TMax)
		{
			float2 UV = t * float2(
				dot(Ray.Direction, CapsuleBounds.SphericalRect.Axis[0]),
				dot(Ray.Direction, CapsuleBounds.SphericalRect.Axis[1])) - float2(
					0.5 * (CapsuleBounds.SphericalRect.x0 + CapsuleBounds.SphericalRect.x1),
					0.5 * (CapsuleBounds.SphericalRect.y0 + CapsuleBounds.SphericalRect.y1)
				);
			float2 Extent = 0.5 * float2(
				CapsuleBounds.SphericalRect.x1 - CapsuleBounds.SphericalRect.x0,
				CapsuleBounds.SphericalRect.y1 - CapsuleBounds.SphericalRect.y0);
			// reject hit if outside of quad extents
			t = all(abs(UV) < Extent) ? t : -1.0;
		}
	}

	// ray points toward the plane and intersect it?
	if (t > Ray.TMin && t < Ray.TMax)
	{
        float CapsuleArea = PI * Radius * (Radius + 0.5 * SourceLength);
		float3 LightPower = GetColor(LightId);
        float3 LightRadiance = LightPower / CapsuleArea;
		float t = CapsuleIntersect(Ray.Direction, TranslatedLightPosition - Ray.Origin, GetdPdv(LightId), Radius2, SourceLength);
		if (t > Ray.TMin && t < Ray.TMax)
		{
			LightRadiance *= float3(0, 1, 0);
		}
		return CreateLightHit(LightRadiance, 0.0, t);
	}

	// missed the bounding rect, test capsule anyway to see if any part of the capsule was missing
	{
        float CapsuleArea = PI * Radius * (Radius + 0.5 * SourceLength);
		float3 LightPower = GetColor(LightId);
        float3 LightRadiance = LightPower / CapsuleArea;
		{
			// exact
			float t = CapsuleIntersect(Ray.Direction, TranslatedLightPosition - Ray.Origin, GetdPdv(LightId), Radius2, SourceLength);
			if (t > Ray.TMin && t < Ray.TMax)
			{
				LightRadiance *= float3(1, 0, 0);
				return CreateLightHit(LightRadiance, 0.0, t);
			}
		}
	}
#else
	float t = CapsuleIntersect(Ray.Direction, TranslatedLightPosition - Ray.Origin, Axis, Radius2, SourceLength);
	if (t > Ray.TMin && t < Ray.TMax)
	{
        float CapsRadius = max(Radius, 0.5 * GetRadius(LightId)); // shrink as we go into the light, but not too much
        float CapsuleArea = PI * CapsRadius * (CapsRadius + 0.5 * SourceLength);
		float3 LightPower = GetColor(LightId);
        float3 LightRadiance = LightPower / CapsuleArea;
		float3 LightDirection = TranslatedLightPosition - Ray.Origin;
		float LightDistanceSquared = dot(LightDirection, LightDirection);
#if CAPSULE_SOLIDANGLE_SAMPLING
		FCapsuleSphericalBounds Bounds = CapsuleGetSphericalBounds(LightDirection, Axis, Radius, SourceLength);
		float Pdf = 1.0 / GetCapsuleBoundsInversePdf(Ray.Direction, Bounds);
#else
		float3 Normal = CapsuleNormal(Ray.Origin + t * Ray.Direction, TranslatedLightPosition, Axis * SourceLength);

		float CosTheta = saturate(-dot(Normal, Ray.Direction));

		float Pdf = CosTheta > 0 ? t * t / (4 * PI * CosTheta * Radius * (Radius + 0.5 * SourceLength)) : 0.0;
#endif
		return CreateLightHit(LightRadiance, Pdf, t);
	}
#endif
	return NullLightHit();
}


FLightSample CapsuleLight_SampleLight(
	int LightId,
	float2 RandSample,
	float3 TranslatedWorldPos,
	float3 WorldNormal
)
{
	// Capsule case
	float3 LightDirection = GetTranslatedPosition(LightId) - TranslatedWorldPos;
    float3 Axis = GetdPdv(LightId);
    float SourceLength = GetSourceLength(LightId);
	float Radius = GetCapsuleRadius(LightId, LightDirection, Axis, SourceLength);
	float Radius2 = Pow2(Radius);

#if CAPSULE_SOLIDANGLE_SAMPLING
    float CapsRadius = max(Radius, 0.5 * min(SourceLength, GetRadius(LightId))); // shrink as we go into the light, but not too much
    float CapsuleArea = PI * CapsRadius * (CapsRadius + 0.5 * SourceLength);
    float3 LightPower = GetColor(LightId);
    float3 LightRadiance = LightPower / CapsuleArea;
	
	FCapsuleSphericalBounds Bounds = CapsuleGetSphericalBounds(LightDirection, Axis, Radius, SourceLength);

	float4 Result = SampleCapsuleBounds(Bounds, RandSample);

	float3 L = Result.xyz;
	float InvPdf = Result.w;
		
	// check direction to account for rays that hit the bounding shape but not the capsule
#if 1
	// optimized check (also more robust for tiny radii)
	float Distance = CapsuleTest(L, LightDirection, Axis, Radius2, SourceLength);
#else
	// exact check (produces artifacts for tiny radii)
	float Distance = CapsuleIntersect(L, LightDirection, Axis, Radius2, SourceLength);
#endif
	if (Distance > 0)
	{
		return CreateLightSample(LightRadiance * InvPdf, rcp(InvPdf), L, Distance);
	}
	// didn't pass the acceptance test -- reject this sample
	return NullLightSample();
#else
	// plain area sampling
	// the caps are two halves of a full sphere
	// the body is a cylinder
	// the common factor of 4*PI is accounted for at the end
	float CapsArea = Radius2;
	float BodyArea = 0.5 * Radius * SourceLength;
	float Prob = CapsArea / (CapsArea + BodyArea);

	float3 LightPoint;
	float3 LightNormal;
	if (RandSample.x < Prob)
	{
		RandSample.x /= Prob;
		// sample the caps
		float4 Result = UniformSampleSphere(float2(RandSample));

		LightPoint = Result.xyz * Radius;
		LightPoint.z += 0.5 * SourceLength * sign(Result.z);

		LightNormal = Result.xyz;
	}
	else
	{
		RandSample.x -= Prob;
		RandSample.x /= 1 - Prob;
		// sample the cylinder body

		float Phi = 2 * PI * RandSample.x;

		LightNormal = float3(cos(Phi), sin(Phi), 0.0);
		LightPoint = float3(Radius * LightNormal.xy, (RandSample.y - 0.5) * SourceLength);
	}
	float3x3 CylinderBasis = GetTangentBasis(Axis);

	float3 LightDirection = GetTranslatedPosition(LightId) - TranslatedWorldPos;
	float3 LocalLightDirection = mul(CylinderBasis, LightDirection); // World To Local

	float CosTheta = saturate(dot(LightNormal, -normalize(LocalLightDirection + LightPoint)));

	LightPoint = mul(LightPoint, CylinderBasis) + LightDirection; // Local to World

	float DistanceSquared = dot(LightPoint, LightPoint);
	float Distance = sqrt(DistanceSquared);
	float3 Direction = LightPoint * rcp(Distance);

	float3 LightPower = GetColor(LightId);
	float3 RadianceOverPdf = LightPower * CosTheta * 4 / DistanceSquared;
	float Pdf = CosTheta > 0 ? DistanceSquared / (4 * PI * CosTheta * (CapsArea + BodyArea)) : 0.0;
	return CreateLightSample(RadianceOverPdf, Pdf, Direction, Distance);
#endif
}
