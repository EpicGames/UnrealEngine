// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
DirectionalLight.usf: Light sampling functions for Directional light implementation
===============================================================================================*/

#pragma once

#include "PathTracingLightCommon.ush"

FLightHit DirectionalLight_TraceLight(FRayDesc Ray, int LightId)
{
	if (Ray.TMax == RAY_DEFAULT_T_MAX) // we hit the background
	{
		float3 LightDirection = GetNormal(LightId);
		float SinThetaMax = GetRadius(LightId); // Sin(Angle / 2)
		float SinThetaMax2 = SinThetaMax * SinThetaMax;
		// See explanation in UniformSampleConeRobust
		float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);

		float CosTheta = saturate(dot(normalize(Ray.Direction), LightDirection));

		// Is ray pointing inside the cone of directions?
		if (1 - CosTheta < OneMinusCosThetaMax)
		{
			// We divide the color of the light by the solid angle so
			// that the overall illumination stays constant as the angle changes.
			float SolidAngle = 2 * PI * OneMinusCosThetaMax;
			float3 Radiance = GetColor(LightId) / SolidAngle;
			return CreateLightHit(Radiance, 1.0 / SolidAngle, RAY_DEFAULT_T_MAX);
		}
	}
	return NullLightHit();
}

FLightSample DirectionalLight_SampleLight(
	int LightId,
	float2 RandSample,
	float3 WorldNormal
)
{
	float SinThetaMax = GetRadius(LightId); // currently sin(Angle / 2)
	float4 DirAndPdf = UniformSampleConeRobust(RandSample, SinThetaMax * SinThetaMax);
	float3 Direction = TangentToWorld(DirAndPdf.xyz, GetNormal(LightId));

	// Because the light is normalized by the solid angle, the radiance/pdf ratio is just the color
	return CreateLightSample(GetColor(LightId), DirAndPdf.w, Direction, RAY_DEFAULT_T_MAX);
}

float DirectionalLight_EstimateLight(
	int LightId,
	float3 WorldNormal,
	bool IsTransmissiveMaterial
)
{
	float3 LightDirection = GetNormal(LightId);
	float3 SurfaceNormal = WorldNormal;

	float NoL = 1.0; // trivial upper bound -- trying to be more accurate appears to reduce performance
	float LightPower = Luminance(GetColor(LightId));
	return LightPower * NoL;
}

// NOTE: unused
FVolumeLightSampleSetup DirectionalLight_PrepareLightVolumeSample(
	int LightId,
	float3 RayOrigin,
	float3 RayDirection,
	float TMin,
	float TMax
)
{
	float3 LightColor = GetColor(LightId);
	float SinThetaMax = max(0.001, GetRadius(LightId)); // Sin(Angle / 2)
	float SinThetaMax2 = SinThetaMax * SinThetaMax;
	float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);
	float SolidAngle = 2 * PI * OneMinusCosThetaMax;
	return CreateUniformSampler(max3(LightColor.x, LightColor.y, LightColor.z) * rcp(SolidAngle), TMin, TMax);
}
