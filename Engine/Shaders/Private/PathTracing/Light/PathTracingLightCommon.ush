// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
LightCommon.usf: Common utility functions for light sampling
===============================================================================================*/

#pragma once

#include "/Engine/Shared/RayTracingTypes.h"
#include "/Engine/Shared/PathTracingDefinitions.h"

uint SceneLightCount;
StructuredBuffer<FPathTracingLight> SceneLights;

struct FLightHit
{
	float3 Radiance;
	float Pdf;
	float HitT;

	bool IsMiss() { return HitT <= 0; }
	bool IsHit() { return HitT > 0; }
};

struct FLightSample
{
	float3 RadianceOverPdf;
	float Pdf;
	float3 Direction;
	float Distance;
};


struct FVolumeLightSampleSetup
{
	float VolumeTMin;      // range of 't' values along the ray that overlap the light (and the volume)
	float VolumeTMax;

	float LightImportance; // estimate of the light radiance for this ray (could be 0 if light does not support equi-angular sampling)

	// Equi-Angular case
	float T0;			   // distance from ray origin to projected light center
	float Height;		   // distance from light center to ray (when negative, use uniform sampler)
	float ThetaMin;        // angle to start of ray interval
	float ThetaMax;        // angle to end of ray interval

	float K;               // normalization constant for the pdf

	bool IsValid() { return VolumeTMin < VolumeTMax; } // NOTE: we assume the range is a subset of the ray's TMin/TMax

	float2 SampleDistance(float RandSample)
	{
		if (Height > 0)
		{
			// equi-angular pdf
			const float TanTheta = tan(lerp(ThetaMin, ThetaMax, RandSample));
			const float T = Height * TanTheta + T0;
			const float PdfLocal = rcp((1 + TanTheta * TanTheta) * K);
			return float2(T, PdfLocal);
		}
		else
		{
			// uniform pdf
			const float T = lerp(VolumeTMin, VolumeTMax, RandSample);
			const float PdfLocal = rcp(K);
			return float2(T, PdfLocal);
		}
	}

	float Pdf(float T)
	{
		if (T > VolumeTMin && T < VolumeTMax)
		{
			if (Height > 0)
			{
				const float TanTheta = (T - T0) / Height;
				return rcp((1 + TanTheta * TanTheta) * K);
			}
			else
			{
				return rcp(K);
			}
		}
		return 0.0;
	}
};

FVolumeLightSampleSetup CreateEquiAngularSampler(float LightImportance, float3 Center, float3 Ro, float3 Rd, float TMin, float TMax)
{
	const float3 V = Center - Ro;
	FVolumeLightSampleSetup Result = (FVolumeLightSampleSetup)0;
	Result.VolumeTMin = TMin;
	Result.VolumeTMax = TMax;
	Result.T0 = dot(V, Rd);
	Result.Height = length(cross(V, Rd));
	// TODO: handle Height == 0 case better, the pdf becomes degenerate in this case
	// but unless the light is parented to the camera exactly, it is hard to see in practice
	const float RcpHeight = Result.Height > 0 ? rcp(Result.Height) : 0.0;
	Result.ThetaMin = atan((TMin - Result.T0) * RcpHeight);
	Result.ThetaMax = atan((TMax - Result.T0) * RcpHeight);
	Result.K = (Result.ThetaMax - Result.ThetaMin) * Result.Height;
	Result.LightImportance = LightImportance *  (Result.ThetaMax - Result.ThetaMin) * RcpHeight;
	return Result;
}

FVolumeLightSampleSetup CreateUniformSampler(float LightImportance, float TMin, float TMax)
{
	FVolumeLightSampleSetup Result = (FVolumeLightSampleSetup)0;
	Result.VolumeTMin = TMin;
	Result.VolumeTMax = TMax;
	Result.LightImportance = LightImportance * (TMax - TMin);
	Result.Height = -1.0; // mark that we don't want equi-angular sampling here
	Result.K = TMax - TMin;
	return Result;
}




FLightHit NullLightHit()
{
	return (FLightHit)0;
}

FLightHit CreateLightHit(float3 Radiance, float Pdf, float HitT)
{
	FLightHit Result;
	Result.Radiance = Radiance;
	Result.Pdf = Pdf;
	Result.HitT = HitT;
	return Result;
}

FLightSample NullLightSample()
{
	return (FLightSample)0;
}

FLightSample CreateLightSample(float3 RadianceOverPdf, float Pdf, float3 Direction, float Distance)
{
	FLightSample Sample;
	Sample.RadianceOverPdf = RadianceOverPdf;
	Sample.Pdf = Pdf;
	Sample.Direction = Direction;
	Sample.Distance = Distance;
	return Sample;
}

FVolumeLightSampleSetup NullVolumeLightSetup()
{
	return (FVolumeLightSampleSetup)0;
}

bool IsEnvironmentLight(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK) == PATHTRACING_LIGHT_SKY;
}

bool IsPointLight(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK) == PATHTRACING_LIGHT_POINT;
}

bool IsDirectionalLight(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK) == PATHTRACING_LIGHT_DIRECTIONAL;
}

bool IsRectLight(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK) == PATHTRACING_LIGHT_RECT;
}

bool IsSpotLight(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK) == PATHTRACING_LIGHT_SPOT;
}

// A light is a physical light if it can be intersected by a ray.
bool IsPhysicalLight(int LightId)
{
	return IsEnvironmentLight(LightId);
}

float3 GetTranslatedPosition(int LightId)
{
	return SceneLights[LightId].TranslatedWorldPosition;
}

float3 GetNormal(int LightId)
{
	return SceneLights[LightId].Normal;
}

float3 GetdPdu(int LightId)
{
	return cross(SceneLights[LightId].Normal, SceneLights[LightId].Tangent);
}

float3 GetdPdv(int LightId)
{
	return SceneLights[LightId].Tangent;
}

float GetWidth(int LightId)
{
	return SceneLights[LightId].Dimensions.x;
}

float GetHeight(int LightId)
{
	return SceneLights[LightId].Dimensions.y;
}

float2 GetRectSize(int LightId)
{
	return SceneLights[LightId].Dimensions.xy;
}

float2 GetCosConeAngles(int LightId)
{
	return SceneLights[LightId].Shaping;
}

float3 GetColor(int LightId)
{
	return SceneLights[LightId].Color;
}

float GetRadius(int LightId)
{
	return SceneLights[LightId].Dimensions.x;
}

float GetSourceLength(int LightId)
{
	return SceneLights[LightId].Dimensions.y;
}

float GetAttenuation(int LightId)
{
	return SceneLights[LightId].Attenuation;
}

float GetRectLightBarnCosAngle(int LightId)
{
	return SceneLights[LightId].Shaping.x;
}

float GetRectLightBarnLength(int LightId)
{
	return SceneLights[LightId].Shaping.y;
}

float2 GetRectLightAtlasUVScale(int LightId)
{
	return SceneLights[LightId].RectLightAtlasUVScale;
}

float2 GetRectLightAtlasUVOffset(int LightId)
{
	return SceneLights[LightId].RectLightAtlasUVOffset;
}

#ifndef USE_ATTENUATION_TERM
	#define USE_ATTENUATION_TERM 1
#endif

float ComputeAttenuationFalloff(float DistanceSquared, int LightId)
{
#if USE_ATTENUATION_TERM
	// Mirrors GetLocalLightAttenuation() custom attenuation controls
	// #dxr_todo: UE-72508: encapsulate this function in a shared space
	float InvAttenuationRadius = GetAttenuation(LightId);
	float NormalizeDistanceSquared = DistanceSquared * Square(InvAttenuationRadius);
	if ((SceneLights[LightId].Flags & PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK) == 0)
	{
		return Square(saturate(1.0 - Square(NormalizeDistanceSquared)));
	}
	else
	{
		// roughly cancel out "native" square distance falloff before applying exponent based falloff function
        // this appears to match the behavior of the deferred lighting passes
		float FalloffExponent = SceneLights[LightId].FalloffExponent;
		return DistanceSquared * pow(1.0 - saturate(NormalizeDistanceSquared), FalloffExponent);
	}
#else
	return 1.0;
#endif
}


#ifndef USE_IES_TERM
#define USE_IES_TERM 1
#endif

float ComputeIESAttenuation(int LightId, float3 TranslatedWorldPosition)
{
	float Result = 1.f;
#if USE_IES_TERM
	if (SceneLights[LightId].IESAtlasIndex >= 0)
	{
		float3 LightDirection = normalize(GetTranslatedPosition(LightId) - TranslatedWorldPosition);

		float DotProd = dot(LightDirection, GetNormal(LightId));
		float Angle = asin(DotProd);
		const float NormAngle = Angle / PI + 0.5f;

		float du = dot(LightDirection, GetdPdu(LightId));
		float dv = dot(LightDirection, GetdPdv(LightId));
		float NormTangentAngle = atan2(dv, du) / (PI * 2.f) + 0.5f;

		const float3 UVW = float3(NormAngle, NormTangentAngle, SceneLights[LightId].IESAtlasIndex);
		Result = View.IESAtlasTexture.SampleLevel(View.IESAtlasSampler, UVW, 0);
	}
#endif
	return Result;
}

bool HasTransmission(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_TRANSMISSION_MASK) != 0;
}

uint GetLightingChannelMask(int LightId)
{
	return SceneLights[LightId].Flags & PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
}

bool IsStationary(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_STATIONARY_MASK) != 0;
}

bool CastsShadow(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_CAST_SHADOW_MASK) != 0;
}

bool CastsVolumeShadow(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_CAST_VOL_SHADOW_MASK) != 0;
}

bool CastsCloudShadow(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_CAST_CLOUD_SHADOW_MASK) != 0;
}

bool HasRectTexture(int LightId)
{
	return (SceneLights[LightId].Flags & PATHTRACER_FLAG_HAS_RECT_TEXTURE_MASK) != 0;
}

float GetVolumetricScatteringIntensity(int LightId)
{
	return SceneLights[LightId].VolumetricScatteringIntensity;
}

float GetLightSpecularScale(int LightId)
{
	return UnpackFloat2FromUInt(SceneLights[LightId].DiffuseSpecularScale).y;
}

float GetLightDiffuseScale(int LightId)
{
	return UnpackFloat2FromUInt(SceneLights[LightId].DiffuseSpecularScale).x;
}

float GetLightIndirectScale(int LightId, float PathRoughness)
{
	// Directly visible hits and sharp reflections should not be scaled, but as the roughness increases, we want to apply the indirect lighting scale
	return lerp(1.0, SceneLights[LightId].IndirectLightingScale, saturate(4.0 * PathRoughness));
}

uint GetLightMissShaderIndex(int LightId)
{
	return SceneLights[LightId].MissShaderIndex;
}
