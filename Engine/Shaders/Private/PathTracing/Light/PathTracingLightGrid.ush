// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef USE_PATH_TRACING_LIGHT_GRID
#define USE_PATH_TRACING_LIGHT_GRID 0
#endif

#include "/Engine/Shared/PathTracingDefinitions.h"


#if USE_PATH_TRACING_LIGHT_GRID

#include "../../Visualization.ush"

// extra shader parameters that must be defined when using the light grid
uint SceneInfiniteLightCount;
float3 SceneLightsTranslatedBoundMin;
float3 SceneLightsTranslatedBoundMax;
Texture2DArray<uint> LightGrid;
Buffer<uint> LightGridData;
uint LightGridResolution;
uint LightGridMaxCount;
uint LightGridAxis;
#endif

struct FLightLoopCount {
	uint NumLights;
	uint NumMISLights;
#if USE_PATH_TRACING_LIGHT_GRID
	uint LightGridOffset;
#endif
};

#if USE_PATH_TRACING_LIGHT_GRID

FLightLoopCount LightGridLookup(float3 TranslatedWorldPos)
{
	FLightLoopCount Result;
	Result.NumLights = SceneInfiniteLightCount;
	Result.NumMISLights = SceneInfiniteLightCount;
	Result.LightGridOffset = ~0u;

	if (all(SceneLightsTranslatedBoundMin <= TranslatedWorldPos) && all(TranslatedWorldPos <= SceneLightsTranslatedBoundMax))
	{
		float3 P = TranslatedWorldPos - SceneLightsTranslatedBoundMin;
		float3 D = SceneLightsTranslatedBoundMax - SceneLightsTranslatedBoundMin;
		int2 UVx = int2(floor(LightGridResolution * P.yz / D.yz));
		int2 UVy = int2(floor(LightGridResolution * P.xz / D.xz));
		int2 UVz = int2(floor(LightGridResolution * P.xy / D.xy));

		uint LightGridNx = LightGrid.Load(int4(UVx, 0, 0));
		uint LightGridNy = LightGrid.Load(int4(UVy, 1, 0));
		uint LightGridNz = LightGrid.Load(int4(UVz, 2, 0));
		uint LightGridN = LightGridNx;
		uint LightGridOffset = LightGridMaxCount * (0 + 3 * (UVx.x + UVx.y * LightGridResolution));
		if ((LightGridAxis >= 3 && (LightGridNy & PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK) < (LightGridN & PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK)) || LightGridAxis == 1)
		{
			LightGridN = LightGridNy;
			LightGridOffset = LightGridMaxCount * (1 + 3 * (UVy.x + UVy.y * LightGridResolution));
		}
		if ((LightGridAxis >= 3 && (LightGridNz & PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK) < (LightGridN & PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK)) || LightGridAxis == 2)
		{
			LightGridN = LightGridNz;
			LightGridOffset = LightGridMaxCount * (2 + 3 * (UVz.x + UVz.y * LightGridResolution));
		}

		Result.LightGridOffset = LightGridOffset;
		Result.NumLights += LightGridN & PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK;
		if ((LightGridN & PATHTRACER_LIGHT_GRID_SINGULAR_MASK) == 0)
		{
			// at least some area lights, enable MIS for this cell
			Result.NumMISLights = Result.NumLights;
		}
	}
	return Result;
}

float3 LightGridVisualize(FLightLoopCount LoopCount, int DebugMode, uint MaxCount)
{
	const float3 OutsideColor = 0.18;
	const float3 EmptyColor = 0.36;

	if (LoopCount.LightGridOffset != ~0u)
	{
		switch (DebugMode)
		{
			case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_LIST_HASH:
			{
				// color by unique light list
				uint H = 0;
				for (int Index = 0, Num = LoopCount.NumLights - SceneInfiniteLightCount; Index < Num; Index++)
				{
					H = StrongIntegerHash(H + LightGridData[LoopCount.LightGridOffset + Index]);
				}
                if (LoopCount.NumLights == SceneInfiniteLightCount)
                {
                    return EmptyColor;
                }
				return HUEtoRGB((H & 0xFFFFFF) * 5.96046447754e-08);
			}
			case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_SINGULAR:
			{
				// color by presence of singular lights
				if (LoopCount.NumLights == SceneInfiniteLightCount)
				{
					return EmptyColor;
				}
				else if (LoopCount.NumLights == LoopCount.NumMISLights)
				{
					return float3(0.2, 0.2, 1); // blue - some area lights
				}
				else
				{
					return float3(0.2, 1, 0.2); // green - only point lights
				}
			}
			case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_AXIS:
			{
				// color by chosen axis
				int Axis = (LoopCount.LightGridOffset / LightGridMaxCount) % 3;
				if (Axis == 0) return float3(0.9, 0.2, 0.2);
				if (Axis == 1) return float3(0.2, 0.9, 0.2);
				if (Axis == 2) return float3(0.2, 0.2, 0.9);
				return float3(0.8, 0.2, 0.8);
			}
			default: 
			{
				// default mode - color by light count
				uint N = LoopCount.NumLights - SceneInfiniteLightCount;
				if (N < 1)
				{
					return EmptyColor;
				}
                if (N > MaxCount)
                {
                    return float3(1, 0.1, 1);
                }
				float Max = LightGridMaxCount;
				float t = saturate(float(N) / MaxCount);
				return BlueGreenRedColorMap(t);
			}
		}
	}
	// outside light grid bounds
	return OutsideColor;
}

uint GetLightId(uint Index, FLightLoopCount LoopCount)
{
	if (Index >= SceneInfiniteLightCount)
	{
		return LightGridData[LoopCount.LightGridOffset + Index - SceneInfiniteLightCount];
	}
	return Index;
}

#else

FLightLoopCount LightGridLookup(float3 TranslatedWorldPos)
{
	uint NumLights = min(SceneLightCount, RAY_TRACING_LIGHT_COUNT_MAXIMUM);
	FLightLoopCount Result;
	Result.NumLights = Result.NumMISLights = NumLights;
	return Result;
}

float3 LightGridVisualize(FLightLoopCount LoopCount, int DebugMode, uint MaxCount)
{
	return 0.5;
}

uint GetLightId(uint Index, FLightLoopCount LoopCount)
{
	return Index;
}

#endif
