// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
LightSampling.usf: Light sampling for next-event estimation 
===============================================================================================*/

#pragma once

#ifndef ENABLE_TRANSMISSION
	#define ENABLE_TRANSMISSION 1
#endif

//#define UNIFORM_LIGHT_SELECTION

#include "PathTracingLightCommon.ush"
#include "PathTracingDirectionalLight.ush"
#include "PathTracingPointLight.ush"
#include "PathTracingRectLight.ush"
#include "PathTracingSkyLight.ush"

// Returns float4 packed with Radiance and Pdf
FLightHit TraceLight(FRayDesc Ray, int LightId)
{
	switch (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK)
	{
		case PATHTRACING_LIGHT_SKY:			return SkyLight_TraceLight(Ray, LightId);
		case PATHTRACING_LIGHT_DIRECTIONAL: return DirectionalLight_TraceLight(Ray, LightId);
		case PATHTRACING_LIGHT_POINT:
		case PATHTRACING_LIGHT_SPOT:		return PointLight_TraceLight(Ray, LightId);
		case PATHTRACING_LIGHT_RECT:		return RectLight_TraceLight(Ray, LightId);
		default:							return NullLightHit(); // unknown light type?
	}
}

// Choose a point on a light as seen from the given shading point
FLightSample SampleLight(
	int LightId,
	float2 RandSample,
	float3 TranslatedWorldPos,
	float3 WorldNormal
)
{
	switch (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK)
	{
		case PATHTRACING_LIGHT_SKY:			return SkyLight_SampleLight(LightId, RandSample, WorldNormal);
		case PATHTRACING_LIGHT_DIRECTIONAL: return DirectionalLight_SampleLight(LightId, RandSample, WorldNormal);
		case PATHTRACING_LIGHT_POINT:
		case PATHTRACING_LIGHT_SPOT:		return PointLight_SampleLight(LightId, RandSample, TranslatedWorldPos, WorldNormal);
		case PATHTRACING_LIGHT_RECT:		return RectLight_SampleLight(LightId, RandSample, TranslatedWorldPos, WorldNormal);
		default:							return NullLightSample(); // unknown light type?
	}
}

float EstimateLight(
	int LightId,
	float3 TranslatedWorldPos,
	float3 WorldNormal,
	uint PrimitiveLightingChannelMask,
	bool IsTransmissiveMaterial
)
{
	if ((GetLightingChannelMask(LightId) & PrimitiveLightingChannelMask) == 0)
	{
		return 0.0;
	}

	switch (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK)
	{
		case PATHTRACING_LIGHT_SKY:			return SkyLight_EstimateLight(LightId, WorldNormal, IsTransmissiveMaterial);
		case PATHTRACING_LIGHT_DIRECTIONAL:	return DirectionalLight_EstimateLight(LightId, WorldNormal, IsTransmissiveMaterial);
		case PATHTRACING_LIGHT_POINT:
		case PATHTRACING_LIGHT_SPOT:		return PointLight_EstimateLight(LightId, TranslatedWorldPos, WorldNormal, IsTransmissiveMaterial);
		case PATHTRACING_LIGHT_RECT:		return RectLight_EstimateLight(LightId, TranslatedWorldPos, WorldNormal, IsTransmissiveMaterial);
		default:							return 0.0;
	}
}

FVolumeLightSampleSetup PrepareLightVolumeSample(
	int LightId,
	float3 RayOrigin,
	float3 RayDirection,
	float TMin,
	float TMax
)
{
	// NOTE: distant lights are excluded from this because they work better with density sampling
	switch (SceneLights[LightId].Flags & PATHTRACER_FLAG_TYPE_MASK)
	{
		//case PATHTRACING_LIGHT_SKY:         return SkyLight_PrepareLightVolumeSample(LightId, RayOrigin, RayDirection, TMin, TMax);
		//case PATHTRACING_LIGHT_DIRECTIONAL: return DirectionalLight_PrepareLightVolumeSample(LightId, RayOrigin, RayDirection, TMin, TMax);
		case PATHTRACING_LIGHT_POINT:
		case PATHTRACING_LIGHT_SPOT:        return PointLight_PrepareLightVolumeSample(LightId, RayOrigin, RayDirection, TMin, TMax);
		case PATHTRACING_LIGHT_RECT:        return RectLight_PrepareLightVolumeSample(LightId, RayOrigin, RayDirection, TMin, TMax);
		default:                            return NullVolumeLightSetup();
	}
}

// #dxr_todo: passing LightPickingCdf as anything but inout causes performance to drop significantly -- why ?
float InitLightPickingCdf(
	float3 TranslatedWorldPos,
	float3 WorldNormal,
	uint PrimitiveLightingChannelMask,
	bool IsTransmissiveMaterial,
	inout float LightPickingCdf[RAY_TRACING_LIGHT_COUNT_MAXIMUM])
{
	// This code assumes SceneLightCount > 0
#ifdef UNIFORM_LIGHT_SELECTION
	return SceneLightCount;
#else
	// Build irradiance estimate prefix sum
	float CdfSum = 0;
	{
		for (uint LightIndex = 0; LightIndex < SceneLightCount; ++LightIndex)
		{
			CdfSum += EstimateLight(LightIndex, TranslatedWorldPos, WorldNormal, PrimitiveLightingChannelMask, IsTransmissiveMaterial);
			LightPickingCdf[LightIndex] = CdfSum;
		}
	}
	return CdfSum;
#endif
}

// #dxr_todo: passing LightPickingCdf as anything but inout causes performance to drop significantly -- why ?
void SelectLight(float RandSample, int NumLights, inout float LightPickingCdf[RAY_TRACING_LIGHT_COUNT_MAXIMUM], out uint LightId, out float LightPickPdf)
{
	// This code assumes SceneLightCount > 0
#ifdef UNIFORM_LIGHT_SELECTION
	LightId = min(int(floor(RandSample)), NumLights - 1);
	LightPickPdf = 1.0;
#else
#if 0
	// linear search
	float PreviousCdfValue = 0;
	for (LightId = 0; LightId < NumLights - 1; ++LightId)
	{
		if (RandSample < LightPickingCdf[LightId])
		{
			break;
		}
		PreviousCdfValue = LightPickingCdf[LightId];
	}

	LightPickPdf = LightPickingCdf[LightId] - PreviousCdfValue;
#else
	// binary search
	LightId = 0;
	for (int Count = NumLights; Count > 0;)
	{
		int Step = Count / 2;
		int Iterator = LightId + Step;
		if (RandSample < LightPickingCdf[Iterator])
		{
			Count = Step;
		}
		else
		{
			LightId = Iterator + 1;
			Count -= Step + 1;
		}
	}
	LightPickPdf = LightPickingCdf[LightId];
	BRANCH if (LightId > 0)
	{
		LightPickPdf -= LightPickingCdf[LightId - 1];
	}
#endif
#endif
}
