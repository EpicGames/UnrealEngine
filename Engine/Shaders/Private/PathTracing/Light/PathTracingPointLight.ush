// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingPointLight.ush: Light sampling functions for Point lights
===============================================================================================*/

#pragma once

#include "PathTracingLightCommon.ush"
#include "PathTracingCapsuleLight.ush"
#include "../../IntersectionUtils.ush"


// #dxr_todo: These are stolen from DynamicLightingCommon.ush but I was having trouble getting that header to include cleanly
/** 
* Calculates attenuation for a spot light.
* L normalize vector to light. 
* SpotDirection is the direction of the spot light.
* SpotAngles.x is CosOuterCone, SpotAngles.y is InvCosConeDifference. 
*/
float SpotAttenuation(float3 L, float3 SpotDirection, float2 SpotAngles)
{
	float ConeAngleFalloff = Square(saturate((dot(L, -SpotDirection) - SpotAngles.x) * SpotAngles.y));
	return ConeAngleFalloff;
}


FLightHit PointLight_TraceLight(FRayDesc Ray, int LightId)
{
	FLightHit Result = NullLightHit();
	if (GetSourceLength(LightId) > 0)
	{
		Result = CapsuleLight_TraceLight(Ray, LightId);
	}
	else
	{
		float3 TranslatedLightPosition = GetTranslatedPosition(LightId);
		float3 oc = Ray.Origin - TranslatedLightPosition;
        float LightDistanceSquared = dot(oc, oc);
		// Clamp the radius such that we are never fully inside the sphere
		float LightRadius2 = min(0.99 * LightDistanceSquared, Pow2(GetRadius(LightId)));

		float b = dot(oc, Ray.Direction);
		// "Precision Improvements for Ray / Sphere Intersection" - Ray Tracing Gems (2019)
		// https://link.springer.com/content/pdf/10.1007%2F978-1-4842-4427-2_7.pdf
		// NOTE: we assume the incoming ray direction is normalized
		float h = LightRadius2 - length2(oc - b * Ray.Direction);
		if (h > 0)
		{
			float t = -b - sqrt(h);
			if (t > Ray.TMin && t < Ray.TMax)
			{
				// #dxr_todo: sphere area is 4*pi*r^2 -- but the factor of 4 is missing for some reason?
				float3 LightPower = GetColor(LightId);
				float3 LightRadiance = LightPower / (PI * LightRadius2);

                float SinThetaMax2 = LightRadius2 / LightDistanceSquared;
				float OneMinusCosThetaMax = SinThetaMax2 < 0.01 ? SinThetaMax2 * (0.5 + 0.125 * SinThetaMax2) : 1 - sqrt(1 - SinThetaMax2);

				float SolidAngle = 2 * PI * OneMinusCosThetaMax;
				Result = CreateLightHit(LightRadiance, 1.0 / SolidAngle, t);
			}
			// #dxr_todo: process inside hit here ...
			// How should we define radiance on the inside of the sphere?
		}
	}
	if (Result.IsHit())
	{
		float3 LightDirection = GetTranslatedPosition(LightId) - Ray.Origin;
		float LightDistanceSquared = dot(LightDirection, LightDirection);
		Result.Radiance *= ComputeIESAttenuation(LightId, Ray.Origin);
		Result.Radiance *= ComputeAttenuationFalloff(LightDistanceSquared, LightId);
		if (IsSpotLight(LightId))
		{
			float3 LightNormal = GetNormal(LightId);
			float2 CosConeAngles = GetCosConeAngles(LightId);
			Result.Radiance *= SpotAttenuation(LightDirection * rsqrt(LightDistanceSquared), LightNormal, CosConeAngles);
		}
	}
	return Result;
}

FLightSample PointLight_SampleLight(
	int LightId,
	float2 RandSample,
	float3 TranslatedWorldPos,
	float3 WorldNormal
)
{
	float3 LightDirection = GetTranslatedPosition(LightId) - TranslatedWorldPos;
	float LightDistanceSquared = dot(LightDirection, LightDirection);

	FLightSample Result = NullLightSample();
	if (GetSourceLength(LightId) > 0)
	{
		Result = CapsuleLight_SampleLight(LightId, RandSample, TranslatedWorldPos, WorldNormal);
	}
	else
	{
		// Point light case

		// Sample the solid angle subtended by the sphere (which could be singular, in which case the PDF will be infinite)
		// Clamp the radius such that we are never fully inside the sphere
		float Radius2 = min(0.99 * LightDistanceSquared, Pow2(GetRadius(LightId)));
        float SinThetaMax2 = Radius2 / LightDistanceSquared;

		// #dxr_todo: find a better way of handling the region inside the light than just clamping to 1.0 here
		float4 DirAndPdf = UniformSampleConeRobust(RandSample, SinThetaMax2);

		float CosTheta = DirAndPdf.z;
		float SinTheta2 = 1.0 - CosTheta * CosTheta;

		Result.Direction = TangentToWorld(DirAndPdf.xyz, normalize(LightDirection));
		Result.Distance = length(LightDirection) * (CosTheta - sqrt(max(SinThetaMax2 - SinTheta2, 0.0)));

		Result.Pdf = DirAndPdf.w;

		float3 LightPower = GetColor(LightId);
		float3 LightRadiance = LightPower / (PI * Radius2);

		// When the angle is very small, Radiance over pdf simplifies even more since SolidAngle ~= PI * SinThetaMax2
		// Canceling out common factors further leads to the classic Power / D^2 formula
		Result.RadianceOverPdf = SinThetaMax2 < 0.001 ? LightPower / LightDistanceSquared : LightRadiance / Result.Pdf;
	}

	// NOTE: uses distance to center to keep fadeoff mask consistent for all samples
	Result.RadianceOverPdf *= ComputeAttenuationFalloff(LightDistanceSquared, LightId);
	Result.RadianceOverPdf *= ComputeIESAttenuation(LightId, TranslatedWorldPos);
	if (IsSpotLight(LightId))
	{
		float3 LightNormal = GetNormal(LightId);
		float2 CosConeAngles = GetCosConeAngles(LightId);
		Result.RadianceOverPdf *= SpotAttenuation(LightDirection * rsqrt(LightDistanceSquared), LightNormal, CosConeAngles);
	}

	return Result;
}

float PointLight_EstimateLight(
	int LightId,
	float3 TranslatedWorldPos,
	float3 WorldNormal,
	bool IsTransmissiveMaterial
)
{
	// Distance
	float3 LightDirection = GetTranslatedPosition(LightId) - TranslatedWorldPos;
	float LightDistanceSquared = dot(LightDirection, LightDirection);

	// Geometric term
	float NoL = 1.0; // trivial upper bound -- trying to be more accurate appears to reduce performance

	float LightPower = Luminance(GetColor(LightId));
	float Falloff = ComputeAttenuationFalloff(LightDistanceSquared, LightId);
	float OutIrradiance = LightPower * Falloff * NoL / LightDistanceSquared;
	if (IsSpotLight(LightId))
	{
		float3 LightNormal = GetNormal(LightId);
		float2 CosConeAngles = GetCosConeAngles(LightId);
		OutIrradiance *= SpotAttenuation(LightDirection * rsqrt(LightDistanceSquared), LightNormal, CosConeAngles);
	}
	return OutIrradiance;
}

FVolumeLightSampleSetup PointLight_PrepareLightVolumeSample(
	int LightId,
	float3 RayOrigin,
	float3 RayDirection,
	float TMin,
	float TMax
)
{
	float3 Center = GetTranslatedPosition(LightId);
	float AttenuationRadius = rcp(GetAttenuation(LightId));
	float2 T = RaySphereOverlap(RayOrigin, RayDirection, Center, AttenuationRadius, TMin, TMax);
	if (T.x < T.y)
	{
		float3 C = GetColor(LightId);
		float LightImportance = max3(C.x, C.y, C.z);
		if (IsSpotLight(LightId))
		{
			float3 N = GetNormal(LightId);
			// now clip against the cone of the spot
			const float CosOuterConeAngle = GetCosConeAngles(LightId).x;

			float2 TCone = RayInfiniteConeVolumeOverlap(RayOrigin, RayDirection, Center, N, CosOuterConeAngle);
			if (TCone.x < TCone.y)
			{
				T.x = max(TCone.x, T.x);
				T.y = min(TCone.y, T.y);
			}
			else
			{
				// hit the sphere but missed the cone
				return NullVolumeLightSetup();
			}

			// find point on the ray which minimizes the dot product against the spot axis
			// This is the largest that the spot attenuation can be for this ray -- this helps with sampling that goes through the edge of the cone when attenuation is strong
			// Given the constants below, the dot product between a point Ro+t*Rd along the ray against the spot's cone axis can be solved by:
			// Solve[D[(A - t*B)/Sqrt[C - 2*t*D + t^2], t] == 0, t]
			// The solution of which is given below. If the denominator is negative, we are 
			float3 Q = Center - RayOrigin;
			float A = dot(Q, N);
			float B = dot(RayDirection, N);
			float C = dot(Q, Q);
			float D = dot(Q, RayDirection);
			float Denom = B * D - A;
			float TClosest = (B * C - A * D) * rcp(Denom);
			float3 Closest = (Q - RayDirection * TClosest) * sign(Denom);
			LightImportance *= SpotAttenuation(normalize(Closest), N, GetCosConeAngles(LightId));
		}
		if ((SceneLights[LightId].Flags & PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK) == 0)
		{
			// inverse square falloff requires equi-angular sampling
			return CreateEquiAngularSampler(LightImportance, Center, RayOrigin, RayDirection, T.x, T.y);
		}
		// otherwise, default to uniform sampling
		return CreateUniformSampler(LightImportance, T.x, T.y);
	}
	return NullVolumeLightSetup();
}
