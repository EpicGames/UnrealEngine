// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RectLight.usf: Light sampling functions for Rect light implementation
===============================================================================================*/

#pragma once

#include "../../RectLight.ush"
#include "PathTracingLightCommon.ush"

#if USE_RECT_LIGHT_TEXTURES

float3 EvaluateTexture(int LightId, float2 UV)
{
	if (HasRectTexture(LightId))
	{
		// TODO: could use path roughness / ray cone to lower the mip level here
		UV.y = 1 - UV.y; // match orientation with raster path
		// Transform local UV into rect. light atlas UV
		const float2 UVScale = GetRectLightAtlasUVScale(LightId);
		const float2 UVOffset = GetRectLightAtlasUVOffset(LightId);
		UV = UV * UVScale + UVOffset;
		return View.RectLightAtlasTexture.SampleLevel(View.RectLightAtlasSampler, UV, 0).xyz;
	}
	return 1.0;
}

#endif

float3 BilinearQuadWarp(float2 uv, float W00, float W01, float W10, float W11) {
	// "Practical Product Sampling by Fitting and Composing Warps" - EGSR 2020
	// https://casual-effects.com/research/Hart2020Sampling/index.html
	// https://www.shadertoy.com/view/wljyDz

	float a = lerp(W00, W01, .5);
	float b = lerp(W10, W11, .5);
	float u = a == b ? uv.x : (sqrt(lerp(a * a, b * b, uv.x)) - a) / (b - a);
	float c = lerp(W00, W10, u);
	float d = lerp(W01, W11, u);
	float v = c == d ? uv.y : (sqrt(lerp(c * c, d * d, uv.y)) - c) / (d - c);
	float area = lerp(a, b, .5);
	float pdf = lerp(c, d, v) / area;
	return float3(u, v, pdf);
}

// TODO: This method should be moved to RectLight.ush
// TODO: Analyze the overall efficiency (MSE reduction vs. time)
//  Projected solid angle sampling converges faster at equal sample count, but is a bit slower
// TODO: Unify with UniformSampleSphericalRect so both return the same struct, making them easier to interchange
float4 SampleApproxProjectionSphericalRect(float2 Rand, FSphericalRect SphericalRect, float3 WorldNormal)
{
	// Construct the quad vertices
	float3 S00 = float3(SphericalRect.x0, SphericalRect.y0, SphericalRect.z0);
	float3 S10 = float3(SphericalRect.x1, SphericalRect.y0, SphericalRect.z0);
	float3 S11 = float3(SphericalRect.x1, SphericalRect.y1, SphericalRect.z0);
	float3 S01 = float3(SphericalRect.x0, SphericalRect.y1, SphericalRect.z0);

	// Compute the cosine of the normal against each corner of the quad
	float3 LightSpaceNormal = mul(SphericalRect.Axis, WorldNormal);
	float W00 = saturate(dot(normalize(S00), LightSpaceNormal));
	float W10 = saturate(dot(normalize(S10), LightSpaceNormal));
	float W11 = saturate(dot(normalize(S11), LightSpaceNormal));
	float W01 = saturate(dot(normalize(S01), LightSpaceNormal));

	// Warp a 2D sample according to the cosine at each corner of the quad
	float3 WarpedRand = BilinearQuadWarp(Rand, W00, W01, W10, W11);

	// Proceed with uniform sampling of the spherical rectangle
	float3 Direction = UniformSampleSphericalRect(WarpedRand.xy, SphericalRect).Direction;
	float OutPdf = isfinite(SphericalRect.SolidAngle) ? 1.0 / SphericalRect.SolidAngle : 0.0;

	// Return Direction vector and adjusted PDF
	return float4(Direction, OutPdf * WarpedRand.z);
}

FLightHit RectLight_TraceLight(FRayDesc Ray, int LightId)
{
	float3 TranslatedLightPosition = GetTranslatedPosition(LightId);
	float3 LightNormal = GetNormal(LightId);
	float3 LightDirection = TranslatedLightPosition - Ray.Origin;
	float DoN = dot(Ray.Direction, LightNormal);
	float t = dot(LightDirection, LightNormal) / DoN;
	// ray points toward the plane and intersect it?
	if (DoN < 0 && t > Ray.TMin && t < Ray.TMax)
	{
		float3 LightdPdu = GetdPdu(LightId);
		float3 LightdPdv = GetdPdv(LightId);
		float2 LightExtent = 0.5 * GetRectSize(LightId);

		float3 P = t * Ray.Direction - LightDirection;
		float2 UV = float2(dot(P, LightdPdu), dot(P, LightdPdv));
		// test point against 
		if (all(abs(UV) <= LightExtent))
		{
			// Clip the Rectangle by the barndoors
			FRect Rect = GetRect(LightDirection,
				-LightNormal,
				LightdPdv,
				LightExtent.x,
				LightExtent.y,
				GetRectLightBarnCosAngle(LightId),
				GetRectLightBarnLength(LightId),
				true);
			P = t * Ray.Direction - Rect.Origin;
			// test again with the clipped extents
			float2 ClippedUV = float2(dot(P, LightdPdu), dot(P, LightdPdv));
			if (all(abs(ClippedUV) <= Rect.Extent))
			{
				// stored color is radiance
				float3 Radiance = GetColor(LightId) * ComputeIESAttenuation(LightId, Ray.Origin);
				Radiance *= ComputeAttenuationFalloff(dot(LightDirection, LightDirection), LightId);
				FSphericalRect SphericalRect = BuildSphericalRect(Rect);
#if USE_RECT_LIGHT_TEXTURES
				Radiance *= EvaluateTexture(LightId, (UV + LightExtent) / (LightExtent * 2));
#endif
				float Pdf = rcp(GetSphericalRectInversePdf(Ray.Direction, t * t, SphericalRect));
				return CreateLightHit(Radiance, Pdf, t);
			}
		}
	}
	return NullLightHit();
}

FLightSample RectLight_SampleLight(
	int LightId,
	float2 RandSample,
	float3 TranslatedWorldPos,
	float3 WorldNormal
)
{
	float3 TranslatedLightPosition = GetTranslatedPosition(LightId);
	float3 LightNormal = GetNormal(LightId);
	float3 LightdPdu = GetdPdu(LightId);
	float3 LightdPdv = GetdPdv(LightId);
	float LightWidth = GetWidth(LightId);
	float LightHeight = GetHeight(LightId);

	// Define rectangle and compute solid angle
	float3 LightDirection = TranslatedLightPosition - TranslatedWorldPos;
	FRect Rect = GetRect(LightDirection,
		-LightNormal,
		LightdPdv,
		0.5 * LightWidth,
		0.5 * LightHeight,
		GetRectLightBarnCosAngle(LightId),
		GetRectLightBarnLength(LightId),
		true /* bComputeVisibleRect */);
	if (!IsRectVisible(Rect) || dot(Rect.Axis[2], Rect.Origin) < 0)
	{
		return NullLightSample();
	}
	FSphericalRect SphericalRect = BuildSphericalRect(Rect);

	float3 Radiance = GetColor(LightId) * ComputeIESAttenuation(LightId, TranslatedWorldPos);
	float DistanceSquared = length2(LightDirection);
	Radiance *= ComputeAttenuationFalloff(DistanceSquared, LightId);
	FSphericalRectSample Sample = UniformSampleSphericalRect(RandSample, SphericalRect);
#if USE_RECT_LIGHT_TEXTURES
	float2 UV = 0.5 * ((2 * Sample.UV - 1) * Rect.Extent + Rect.Offset) / Rect.FullExtent + 0.5;
	Radiance *= EvaluateTexture(LightId, UV);
#endif
	return CreateLightSample(Radiance * Sample.InvPdf, rcp(Sample.InvPdf), Sample.Direction, Sample.Distance);
}

float RectLight_EstimateLight(
	int LightId,
	float3 TranslatedWorldPos,
	float3 WorldNormal,
	bool IsTransmissiveMaterial
)
{
	// Distance to centroid
	// #dxr_todo: UE-72533 Use closest point, instead
	float3 LightDirection = GetTranslatedPosition(LightId) - TranslatedWorldPos;
	float LightDistanceSquared = dot(LightDirection, LightDirection);
	float3 LightNormal = GetNormal(LightId);

	// Is the shading point behind the light?
	float LNoL = saturate(-dot(normalize(LightDirection), LightNormal));
	if (LNoL <= 0.0)
	{
		return 0.0;
	}
	// Approximate geometric term
	float Width = GetWidth(LightId);
	float Height = GetHeight(LightId);

	// Don't bother trying to bound the N.L term as its in [0,1] and hard to estimate accurately and quickly
	float NoL = 1.0;
	float Area = Width * Height;

	float LightPower = Luminance(GetColor(LightId));
	float Falloff = ComputeAttenuationFalloff(LightDistanceSquared, LightId);
	return LightPower * Falloff * Area * NoL * LNoL / LightDistanceSquared;
}

void ClipRayByPlane(float3 RayOrigin, float3 RayDirection, float3 N, float3 C, inout float TMin, inout float TMax)
{
	const float DoN = dot(RayDirection, N);
	const float TPlane = dot(C - RayOrigin, N) * rcp(DoN);
	if (DoN > 0.0)
	{
		// plane is pointing in the same direction as the ray, clip TMin
		TMin = max(TMin, TPlane);
	}
	if (DoN < 0.0)
	{
		// plane is pointing towards the ray, clip TMax
		TMax = min(TMax, TPlane);
	}
}

FVolumeLightSampleSetup RectLight_PrepareLightVolumeSample(
	int LightId,
	float3 RayOrigin,
	float3 RayDirection,
	float TMin,
	float TMax
)
{
	float3 Center = GetTranslatedPosition(LightId);
	float AttenuationRadius = rcp(GetAttenuation(LightId));
	float2 T = RaySphereOverlap(RayOrigin, RayDirection, Center, AttenuationRadius, TMin, TMax);
	if (T.x < T.y)
	{
		// now clip the sphere of influence against the plane of the quad and adjust the near/far distance depending on the sign
		float3 N = GetNormal(LightId);

		// NOTE: push quad's plane slightly forward to avoid samples that would lie too close to the light
		ClipRayByPlane(RayOrigin, RayDirection, N, Center + N * 0.001, T.x, T.y);

		const float3 C = GetColor(LightId);
		const float Width = GetWidth(LightId);
		const float Height = GetHeight(LightId);
#if 1
		const float BarnCos = GetRectLightBarnCosAngle(LightId);
		const float BarnLen = GetRectLightBarnLength(LightId);
		if (BarnCos > 0.035)
		{
			// Clip ray against barndoor penumbra planes if barndoors are active (see threshold in GetRect)
			const float BarnSin = sqrt(1 - BarnCos * BarnCos);
			const float2 BoundingPlaneX = float2(Width  + BarnLen * BarnSin, BarnLen * BarnCos);
			const float2 BoundingPlaneY = float2(Height + BarnLen * BarnSin, BarnLen * BarnCos);

			const float3 LightdPdu = GetdPdu(LightId);
			const float3 LightdPdv = GetdPdv(LightId);

			ClipRayByPlane(RayOrigin, RayDirection, BoundingPlaneX.x * N + BoundingPlaneX.y * LightdPdu, Center - LightdPdu * Width * 0.5, T.x, T.y);
			ClipRayByPlane(RayOrigin, RayDirection, BoundingPlaneX.x * N - BoundingPlaneX.y * LightdPdu, Center + LightdPdu * Width * 0.5, T.x, T.y);
			ClipRayByPlane(RayOrigin, RayDirection, BoundingPlaneY.x * N + BoundingPlaneY.y * LightdPdv, Center - LightdPdv * Height * 0.5, T.x, T.y);
			ClipRayByPlane(RayOrigin, RayDirection, BoundingPlaneY.x * N - BoundingPlaneY.y * LightdPdv, Center + LightdPdv * Height * 0.5, T.x, T.y);
		}
#endif

		const float Area = Width * Height;
		const float LightImportance = max3(C.x, C.y, C.z) * Area;
		if ((SceneLights[LightId].Flags & PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK) == 0)
		{
			// inverse square falloff requires equi-angular sampling
			return CreateEquiAngularSampler(LightImportance, Center, RayOrigin, RayDirection, T.x, T.y);
		}
		// otherwise, default to uniform sampling
		return CreateUniformSampler(LightImportance, T.x, T.y);
	}
	return NullVolumeLightSetup();
}
