// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
SkyLight.usf: Light sampling functions for SkyLight implementation
===============================================================================================*/

#pragma once

#include "../../RayTracing/RayTracingSkyLightCommon.ush"

// By default, assume MIS is performed in the integrator.
// However in some cases like RTGI we may want to have _some_ basic MIS
// without introducing complexity in the integrator.
#ifndef PATHTRACING_SKY_MIS
#define PATHTRACING_SKY_MIS 0
#endif

float SkyLight_EstimateLight(
	int LightId,
	float3 WorldNormal,
	bool IsTransmissiveMaterial
)
{
#if PATHTRACING_SKY_MIS == 0
	return SkyLight_Estimate();
#else
	return 2 * PI;
#endif
}

FLightHit SkyLight_TraceLight(FRayDesc Ray, int LightId)
{
	if (Ray.TMax == RAY_DEFAULT_T_MAX)
	{
		float4 Result = SkyLight_EvalLight(Ray.Direction);
		return CreateLightHit(Result.xyz, Result.w, RAY_DEFAULT_T_MAX);
	}
	return NullLightHit();
}

FLightSample SkyLight_SampleLight(
	int LightId,
	float2 RandSample,
	float3 WorldNormal
)
{
#if PATHTRACING_SKY_MIS == 0
	FSkyLightSample Sample = SkyLight_SampleLight(RandSample);
	return CreateLightSample(Sample.Radiance / Sample.Pdf, Sample.Pdf, Sample.Direction, RAY_DEFAULT_T_MAX);
#else
	// account for the fact that with MIS compensation, we could have a 0 probability of choosing the sky
	float SkyLightSamplingStrategyPdf = SkyLight_Estimate() > 0.0 ? 0.5 : 0.0;

	// Do a simple one-sample MIS between the skylight and cosine sampling on the assumption that
	// the shading loop is only sampling the light and not doing its own MIS.
	float3 SkyLightRadiance = 0;
	float3 Direction = 0;
	float CosinePdf = 0;
	float SkyLightPdf = 0;
	BRANCH if (RandSample.x < SkyLightSamplingStrategyPdf)
	{
		RandSample.x /= SkyLightSamplingStrategyPdf;

		FSkyLightSample SkySample = SkyLight_SampleLight(RandSample);

		SkyLightRadiance = SkySample.Radiance;
		Direction = SkySample.Direction;
		SkyLightPdf = SkySample.Pdf;
		CosinePdf = saturate(dot(WorldNormal, Direction)) / PI;
	}
	else
	{
		RandSample.x = (RandSample.x - SkyLightSamplingStrategyPdf) / (1.0 - SkyLightSamplingStrategyPdf);

		float4 CosSample = CosineSampleHemisphere(RandSample, WorldNormal);
		
		Direction = CosSample.xyz;
		CosinePdf = CosSample.w;

		float4 SkyEval = SkyLight_EvalLight(Direction);
		SkyLightRadiance = SkyEval.xyz;
		SkyLightPdf = SkyEval.w;
	}

	float Pdf = lerp(CosinePdf, SkyLightPdf, SkyLightSamplingStrategyPdf);

	return CreateLightSample(SkyLightRadiance / Pdf, Pdf, Direction, RAY_DEFAULT_T_MAX);
#endif
}

// NOTE: unused
FVolumeLightSampleSetup SkyLight_PrepareLightVolumeSample(
	int LightId,
	float3 RayOrigin,
	float3 RayDirection,
	float TMin,
	float TMax
)
{
	return CreateUniformSampler(4 * PI, TMin, TMax);
}
