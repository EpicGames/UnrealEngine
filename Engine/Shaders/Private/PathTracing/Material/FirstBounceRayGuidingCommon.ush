// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef USE_FIRST_BOUNCE_RAY_GUIDING
	#define USE_FIRST_BOUNCE_RAY_GUIDING 0
#endif

#define TEXEL_CLUSTER_SIZE 8
#define NUM_CLUSTERS_PER_DIM (64 / TEXEL_CLUSTER_SIZE)
#define NUM_CLUSTERS_PER_TILE (64 * 64 / (TEXEL_CLUSTER_SIZE * TEXEL_CLUSTER_SIZE))

#define DIRECTIONAL_BINS_ONE_DIM 16
#define CDF_TILE_SIZE (NUM_CLUSTERS_PER_DIM * DIRECTIONAL_BINS_ONE_DIM)
