// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	PathTracingEye.usf: Path tracing BRDF model for the eye material
===============================================================================================*/

#pragma once

#include "PathTracingMaterialCommon.ush"
#include "PathTracingFresnel.ush"
#include "PathTracingEnergyConservation.ush"

struct FEyeData
{
	float3x3 Basis;
	float2 Alpha;
	float3 V;

	FBxDFEnergyTermsRGB Spec;
	float3 DiffWeight;
	float LobeProb;
};

FEyeData PrepareEyeData(FPathTracingPayload Payload, float3 V_World)
{
	FEyeData Data = (FEyeData)0;

	Data.Basis = GetTangentBasis(Payload.WorldNormal);
	Data.Alpha = GetGGXAlpha(Payload.GetEyeRoughness());
	Data.V = mul(Data.Basis, V_World);

	const float NoV = saturate(Data.V.z);
	Data.Spec = ComputeGGXSpecEnergyTermsRGB(Payload.GetEyeRoughness(), NoV, Payload.SpecularColor);

	Data.DiffWeight = (1 - Luminance(Data.Spec.E)) * Payload.DiffuseColor;

	// Probability of picking diffuse lobe vs. specular lobe
	Data.LobeProb = LobeSelectionProb(Data.DiffWeight, Data.Spec.E);

	return Data;
}

FMaterialEval Eye_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const FEyeData Data = PrepareEyeData(Payload, V_World);

	// move vectors into right shading frame
	const float3 V = Data.V;
	const float3 L = mul(Data.Basis, L_World);
	const float3 H = normalize(V + L);

	const float NoV = saturate(V.z);
	const float NoL = saturate(L.z);
	const float VoH = saturate(dot(V, H));


	FMaterialEval Result = NullMaterialEval();

	const float IrisMask = Payload.GetEyeIrisMask();
	const float3 IrisNormal = Payload.GetEyeIrisNormal();
	const float3 CausticNormal = Payload.GetEyeCausticNormal();
	const float IrisNoL = saturate(dot(IrisNormal, L_World));
	const float Power = lerp(12, 1, IrisNoL);
	const float Caustic = 0.8 + 0.2 * (Power + 1) * pow(saturate(dot(CausticNormal, L_World)), Power);
	const float Iris = IrisNoL * Caustic;
	const float Sclera = NoL;
	const float EyeDiffuseTweak = 2 * lerp(Sclera, Iris, IrisMask);
	const float DiffPdf = 1 / (2 * PI);

	// Diffuse Lobe
	Result.AddLobeWithMIS(Data.DiffWeight * EyeDiffuseTweak * DiffuseSpecularScale.x, DiffPdf, Data.LobeProb);

	// Specular lobe
	const float2 GGXResult = GGXEvalReflection(L, V, H, Data.Alpha);
	const float3 F = F_Schlick(Payload.SpecularColor, VoH);
	const float3 SpecWeight = F * GGXResult.x * Data.Spec.W;
	const float SpecPdf = GGXResult.y;

	Result.AddLobeWithMIS(SpecWeight * DiffuseSpecularScale.y, SpecPdf, 1.0 - Data.LobeProb);

	Result.Weight *= Payload.BSDFOpacity;

	return Result;
}

FMaterialSample Eye_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	const FEyeData Data = PrepareEyeData(Payload, V_World);

	const float3 V = Data.V;
	const float NoV = saturate(V.z);

	// Randomly choose to sample diffuse or specular
	float3 L = 0, H = 0;
	const bool bSampledDiffuse = RandSample.x < Data.LobeProb;
	if (bSampledDiffuse)
	{
		RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, Data.LobeProb);

		L = UniformSampleHemisphere(RandSample.xy).xyz;
		H = normalize(L + V);
	}
	else
	{
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeProb, 1.0);

		H = ImportanceSampleVisibleGGX(RandSample.xy, Data.Alpha, V).xyz;
		L = reflect(-V, H);
		if (L.z <= 0)
		{
			// invalid output direction, exit early
			return NullMaterialSample();
		}
	}

	// With a valid direction in hand -- now evaluate the BxDF (taking advantage of already computed terms)

	// transform to world space
	const float3 L_World = normalize(mul(L, Data.Basis));

	const float NoL = saturate(L.z);
	const float VoH = saturate(dot(V, H));

	const float2 GGXResult = GGXEvalReflection(L, V, H, Data.Alpha);
	const float SpecPdf = GGXResult.y;
	const float DiffPdf = 1 / (2 * PI);

	FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, 1.0, 1.0, PATHTRACER_SCATTER_DIFFUSE);
	if (bSampledDiffuse)
	{
		const float IrisMask = Payload.GetEyeIrisMask();
		const float3 IrisNormal = Payload.GetEyeIrisNormal();
		const float3 CausticNormal = Payload.GetEyeCausticNormal();
		const float IrisNoL = saturate(dot(IrisNormal, L_World));
		const float Power = lerp(12, 1, IrisNoL);
		const float Caustic = 0.8 + 0.2 * (Power + 1) * pow(saturate(dot(CausticNormal, L_World)), Power);
		const float Iris = IrisNoL * Caustic;
		const float Sclera = NoL;

		const float EyeDiffuseTweak = 2 * lerp(Sclera, Iris, IrisMask);

		// Diffuse Lobe
		Result.AddLobeWithMIS(Data.DiffWeight * EyeDiffuseTweak, DiffPdf, Data.LobeProb);

		Result.Pdf += (1 - Data.LobeProb) * SpecPdf;
	}
	else
	{
		// Specular lobe
		const float3 F = F_Schlick(Payload.SpecularColor, VoH);
		const float3 SpecWeight = F * GGXResult.x * Data.Spec.W;
		
		Result.AddLobeWithMIS(SpecWeight, SpecPdf, 1.0 - Data.LobeProb);
		Result.Pdf += Data.LobeProb * DiffPdf;
		Result.Roughness = Payload.GetEyeRoughness();
		Result.ScatterType = PATHTRACER_SCATTER_SPECULAR;
	}
	Result.Weight *= Payload.BSDFOpacity;
	return Result;
}
