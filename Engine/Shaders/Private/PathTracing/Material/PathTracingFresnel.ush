// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingFresnel.usf: Fresnel utilities for material sampling functions 
===============================================================================================*/
#pragma once

// The functions in this file obey the following convention:
// Eta = bFrontFacing ? Ior : rcp(Ior);
// In other words, when we "enter" the material, we use the IOR (a value greater than 1.0), and when we exit, we use the inverse (a value less than 1.0)

float FresnelReflectance(float CosI, float Eta)
{
	// This is a float only version of F_Fresnel that takes the Ior ratio as input directly (and handles TIR)

	// NOTE: this function tries uses the exact same math as the version below for consistency

	float g2 = Eta * Eta - 1 + CosI * CosI;

	if (g2 >= 0)
	{
		float c = abs(CosI);
		float g = sqrt(g2);
		float a2 = Pow2((g - c) / (g + c));
		float b2 = Pow2((c * (g + c) - 1) / (c * (g - c) + 1));
		return 0.5 * a2 * (1.0 + b2);
	}
	// TIR -- must become fully reflective because refraction direction is not defined
	return 1.0;
}

// TIR handled by Eta parameter, reflectivity by F0
float FresnelReflectance(float CosI, float Eta, float F0)
{
	// Eta = bFrontFacing ? Ior : rcp(Ior);
	float F90 = saturate(F0 * 50.0);

	if (Eta >= 1.0)
	{
		// no TIR possible when "entering" (lower ior -> higher ior)
		return lerp(F0, F90, Pow5(1 - CosI));
	}
	float g2 = Eta * Eta - 1 + CosI * CosI;
	if (g2 >= 0.0)
	{
		return lerp(F0, F90, Pow5(1.0 - sqrt(g2) * rcp(Eta)));
	}
	// TIR -- must become fully reflective because refraction direction is not defined
	return 1.0;
}

// returns true if we sampled a refraction, or false if we reflected (or TIR occured)
bool SampleRefraction(float3 RayDirection, float3 Normal, float Eta, float RandSample, out float3 OutDirection, out float OutFresnel)
{
	// Compute refracted direction and Fresnel term at the same time to re-use intermediate values
	// and to get matching results for the TIR condition
	float CosI = abs(dot(RayDirection, Normal));
	float g2 = Eta * Eta - 1.0 + CosI * CosI;

	if (g2 >= 0.0)
	{
		float c = CosI;
		float g = sqrt(g2);
		float a2 = Pow2((g - c) / (g + c));
		float b2 = Pow2((c * (g + c) - 1) / (c * (g - c) + 1));
		float Fr = 0.5 * a2 * (1.0 + b2);

		// Stochastically decide to transmit or not based on Fr
		// This assumes RandSample is in [0,1)
		if (RandSample >= Fr)
		{
			OutFresnel = 1 - Fr;
			OutDirection = normalize(RayDirection + Normal * (c - g));
			return true;
		}
		OutFresnel = Fr;
	}
	else
	{
		// TIR
		OutFresnel = 1.0;
	}
	// either TIR, or sampled reflection
	OutDirection = RayDirection + (2 * CosI) * Normal;
	return false;
}

// Sample refraction where bending is driven by Eta and reflectivity by F0
bool SampleRefraction(float3 RayDirection, float3 Normal, float Eta, float F0, float RandSample, out float3 OutDirection, out float OutFresnel)
{
	float CosI = abs(dot(RayDirection, Normal));
	float g2 = Eta * Eta - 1.0 + CosI * CosI;
	if (g2 >= 0.0)
	{
		// refraction is possible, compute fresnel
		float g = sqrt(g2);
		float c = Eta >= 1.0 ? CosI : g * rcp(Eta);
		float F90 = saturate(F0 * 50.0);
		float Fr = lerp(F0, F90, Pow5(1 - c));
		// Stochastically decide to transmit or not based on Fr
		// This assumes RandSample is in [0,1)
		if (RandSample >= Fr)
		{
			OutFresnel = 1 - Fr;
			OutDirection = normalize(RayDirection + Normal * (CosI - g));
			return true;
		}
		// sampled reflection
		OutFresnel = Fr;
	}
	else
	{
		// TIR (no refraction possible, so everything must be reflected
		OutFresnel = 1.0;
	}
	// either TIR, or sampled reflection
	OutDirection = RayDirection + (2 * CosI) * Normal;
	return false;
}


#define CLEAR_COAT_F0	0.04 // From hard-coded value in ShadingModels.ush

float ClearCoatFresnel(float VoH)
{
	// from hardcoded logic in ClearCoatBxDF
	return CLEAR_COAT_F0 + (1.0 - CLEAR_COAT_F0) * Pow5(1 - VoH);
}


struct FThinSlabWeights
{
	float3 Reflected;
	float3 Transmitted;
};

// return the amount of reflected and transmitted energy for an infinite slab viewed from a particular direction
// taking into account the (repeated) Fresnel terms and the transmittance through the slab
FThinSlabWeights ComputeThinSlabWeights(float3 SlabColor, float CosV, float Eta, float F0)
{
	// Compute Fresnel reflection off the top of the slab
	// The Fresnel refletion at the bottom layer is exactly equal so we can assume energy is split into R and T at each bounce
	// NOTE: we don't need to pass Eta through here because TIR (which could only occur in contrived scenarios) isn't really supported
	const float R = FresnelReflectance(CosV, 1.0, F0);

	// The slab color is normalized to unit thickness, so we only need to account for the refracted viewing angle
	// Note that we only allow Eta > 1.0 here so that we don't need to worry about TIR
	const float3 A = pow(SlabColor, Eta > 1.0 ? Eta * rsqrt(Eta * Eta - 1.0 + CosV * CosV) : rcp(abs(CosV)));

	// figure out how much is reflected/transmitted from a single interaction
	float3 SlabR = R;
	float3 SlabT = A - R * A; // (1 - R) * A

#if 1
	// Enable bounces within the slab
	if (R < 1.0)
	{
		// If there was some transmission we must also account for the infinite series of bounces within the slab
		// which can be computed exactly with a geometric series. Note that this causes a bit of energy loss because we don't
		// account for this non-linear fresnel term in the energy conservation calculation. However, this is fairly minimal in
		// practice, and the added visual richness seems worth it.
		float3 InvDenom = rcp(1.0 - Pow2(R * A));
		SlabR += R * Pow2(SlabT) * InvDenom;
		SlabT *= InvDenom - R * InvDenom; // (1 - R) * InvDenom
	}
#endif

	FThinSlabWeights Result = (FThinSlabWeights)0;
	Result.Reflected = SlabR;
	Result.Transmitted = SlabT;
	return Result;
}

float ComputeThinTransmittedRoughness(float Roughness, float Ior)
{
	// Transmitted lobe should have slightly less roughness due to the squeeze implied by the IOR (eventually converging to 0.0 as Ior approaches 1.0)
	return Ior > 0.0 ? Roughness * sqrt(1.0 - min(Ior, rcp(Ior))) : Roughness;
}
