// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
Glossy.usf: Microfacet GGX BRDF sampling functions 
===============================================================================================*/

#pragma once

#include "../../BRDF.ush"
#include "../../MonteCarlo.ush"

// be conservative since the GPU may be aggressive with floating point optimizations
#define GGX_MIN_ROUGHNESS 0.01
#define GGX_MIN_ANISOTROPY 0.001 // ignore anisotropy below this amount

float2 GetGGXAlpha(float Roughness)
{
	Roughness = max(Roughness, GGX_MIN_ROUGHNESS);
	return Roughness * Roughness;
}

float2 GetGGXAlpha(float Roughness, float Anisotropy)
{
	Roughness = max(Roughness, GGX_MIN_ROUGHNESS);
	float2 Alpha = 0.0;
	if (abs(Anisotropy) >= GGX_MIN_ANISOTROPY)
	{
		GetAnisotropicRoughness(Roughness * Roughness, Anisotropy, Alpha.x, Alpha.y);
	}
	else
	{
		Alpha = Roughness * Roughness;
	}
	return Alpha;
}

float3x3 GetGGXBasis(float Roughness, float Anisotropy, float3 WorldNormal, float3 WorldTangent, out float2 Alpha)
{
	Roughness = max(Roughness, GGX_MIN_ROUGHNESS);
	if (abs(Anisotropy) >= GGX_MIN_ANISOTROPY)
	{
		// Material is anisotropic, our shading frame should use the tangent vector
		GetAnisotropicRoughness(Roughness * Roughness, Anisotropy, Alpha.x, Alpha.y);
		float3 Y = cross(WorldNormal, WorldTangent);
		float LenY2 = length2(Y);
		if (LenY2 > 0)
		{
			// Tangent vector is valid
			Y *= rsqrt(LenY2);
			return float3x3(cross(Y, WorldNormal), Y, WorldNormal);
		}
		// Fallthrough to build a default basis if we couldn't build one
	}
	else
	{
		// No anisotropy, use simpler formula
		Alpha = Roughness * Roughness;
	}
	// pick an arbitrary basis (no anisotropy, or invalid tangent vector)
	return GetTangentBasis(WorldNormal);
}

float3x3 GetGGXBasis(float Roughness, float3 WorldNormal, out float2 Alpha)
{
	// No anisotropy, just pick an arbitrary basis
	Roughness = max(Roughness, GGX_MIN_ROUGHNESS);
	Alpha = Roughness * Roughness;
	return GetTangentBasis(WorldNormal);
}

// Compute GGX lobe Weight and Pdf (without Fresnel term) given a set of vectors in local space (Z up)
float2 GGXEvalReflection(float3 L, float3 V, float3 H, float2 Alpha, bool bLimitVDNFToReflection = true)
{
	const float NoL = saturate(L.z);
	const float NoV = saturate(V.z);

	if (NoL > 0 && NoV > 0)
	{
		const float D = D_GGXaniso(Alpha.x, Alpha.y, H.z, H.x, H.y);
		// See implementation in Vis_SmithJointAniso for G2/(4*NoV*NoL)
		// We can simplify a bit further since we need both the weight G2/G1 and the pdf
		const float LenL = length(float3(L.xy * Alpha, NoL));
		const float LenV = length(float3(V.xy * Alpha, NoV));
		float k = 1.0;
#if GGX_BOUNDED_VNDF_SAMPLING
		if (bLimitVDNFToReflection)
		{
			float a = saturate(min(Alpha.x, Alpha.y));
			float s = 1.0f + length(V.xy);
			float a2 = a * a, s2 = s * s;
			k = (s2 - a2 * s2) / (s2 + a2 * NoV * NoV); // Eq. 5
		}
#endif
		const float Weight = NoL * (LenV + k * NoV) / (NoV * LenL + NoL * LenV);
		const float Pdf = 0.5 * D * rcp(LenV + k * NoV);
		
		return float2(Weight, Pdf);
	}
	return 0;
}

// Compute GGX lobe Weight and Pdf (without Fresnel term) given a set of vectors in local space (Z up)
float2 GGXEvalRefraction(float3 L, float3 V, float3 H, float2 Alpha, float Eta)
{
	const float NoV = saturate(V.z);
	const float NoL = saturate(-L.z); // NOTE: L should point down

	// This checks that V and L point in opposite sides and that the half-vector is valid
	if (NoL > 0 && NoV > 0 && H.z > 0)
	{
		const float D = D_GGXaniso(Alpha.x, Alpha.y, H.z, H.x, H.y);

		const float VoH = saturate( dot(V, H));
		const float LoH = saturate(-dot(L, H));

		// Same as logic above, only the pdf math differs slightly
		const float LenL = length(float3(L.xy * Alpha, NoL));
		const float LenV = length(float3(V.xy * Alpha, NoV));
		const float Weight = NoL * (LenV + NoV) / (NoV * LenL + NoL * LenV);
		const float Pdf = (LoH * VoH * Eta * Eta * 2.0 * D) / (Pow2(VoH - Eta * LoH) * (LenV + NoV));

		// NOTE: filter out potential NaNs from Pdf
		return float2(Weight, max(Pdf, 0.0));
	}
	return 0;
}
