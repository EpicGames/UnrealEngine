// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingMaterialCommon.ush"

#define PATH_TRACER_HAIR_SHADING_MODEL				1 // 0: zinke diffuse (a simple, cheap model for performance testing)
                                                      // 1: chiang principled hair (a sophisticated model with specular reflection/transmission)


// Shared routines

// wrapper function to get hair strand direction, it is encoded in `WorldSmoothNormal` in offline path tracer
float3 GetHairTangent(in const FPathTracingPayload Payload)
{
	return Payload.WorldSmoothNormal;
}

float3x3 GetHairBasis(float3 T_World, float3 V_World)
{
	// a convenient basis for hair shading calculations:
	//   Z Axis = Hair Tangent
	//   Y Axis = point toward the viewer
	//   X Axis = left/right from viewing direction
	float3 Tx = normalize(cross(T_World, V_World));
	float3 Ty = cross(Tx, T_World);
	return float3x3(Tx, Ty, T_World);
}


#if PATH_TRACER_HAIR_SHADING_MODEL == 0

float ZinkeHairDiffuse(float3 L)
{
	// Evaluate the zinke diffuse lobe model. This is the far field average of a diffuse (lambertian) cylinder
	// This was originally derived in: "Light Scattering from Filaments" - Arno Zinke and Andreas Weber - 2007
	// The exact form was mentioned in: "Importance Sampling for Physically-Based Hair Fiber Models" - Eugene d’Eon, Steve Marschner, Johannes Hanika - 2013
	// 
	const float CosTheta = sqrt(saturate(1 - L.z * L.z));
#if 0
	// exact form - Eq(9) from d'Eon et al.
	const float CosPhi = clamp(L.y / CosTheta, -1.0, 1.0);
	const float SinPhi = sqrt(1.0 - CosPhi * CosPhi);
	const float Phi = acos(CosPhi);
	return ((PI - Phi) * CosPhi + SinPhi) * CosTheta / (4 * PI);
#else
	// optimized fit to avoid acos
	// error is around ~0.001 
	const float CosPhiCosTheta = L.y;
	return 0.0786133 * CosTheta + CosPhiCosTheta * (0.125 + 0.0463867 * CosPhiCosTheta * rcp(CosTheta + 1e-12));
#endif
}

FMaterialSample Hair_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample)
{
	const float3x3 HairBasis = GetHairBasis(GetHairTangent(Payload), V_World);

	// sample a random position across the width of the fiber
	const float h = 2 * RandSample.x - 1;
	// create a normal vector in the plane tangent to the fiber
	const float2 N = float2(h, sqrt(1 - h * h));
	// sample a cosine weighted direction
	const float4 SampledValue = CosineSampleHemisphere(RandSample.yz);
	// rotate this sampled direction about the (random) normal vector
	const float3 L = float3(SampledValue.x * N.y + SampledValue.z * N.x,
			                SampledValue.z * N.y - SampledValue.x * N.x,
		                    SampledValue.y);
	// finally transform into world space
	const float3 L_World = mul(L, HairBasis);

	const float Pdf = ZinkeHairDiffuse(L);
	return CreateMaterialSample(L_World, Payload.GetBaseColor(), Pdf, sign(dot(Payload.WorldGeoNormal, L_World)), 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval Hair_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const float3x3 HairBasis = GetHairBasis(GetHairTangent(Payload), V_World);

	const float3 L = mul(HairBasis, L_World);

	const float Pdf = ZinkeHairDiffuse(L);
	return CreateMaterialEval(Payload.GetBaseColor() * DiffuseSpecularScale.x, Pdf);
}


#elif PATH_TRACER_HAIR_SHADING_MODEL == 1

// The implementation below is based on the following paper:
// "A Practical and Controllable Hair and Fur Model for Production Path Tracing" 
// Matt Jen-Yuan Chiang, Benedikt Bitterli, Chuck Tappan, Brent Burley
// EGSR 2016
// https://media.disneyanimation.com/uploads/production/publication_asset/152/asset/eurographics2016Fur_Smaller.pdf


float LogI0(float x)
{
	// I0 grows too quickly to evaluate robustly in single precision arithmetic
	// instead evaluate the log which is much smoother
	// The fits below were found using Mathematica and have error below 10^-6
	x = abs(x);
	if (x < 16.315929055555998) // transition point where the two approximations line up
	{
		// rational approximation
		// 9 fmas + 2 mults + 1 div
		float num = 0.249915 + x * (0.124919 + x * (0.0486428 + x * (0.011457 + x * 0.000647177)));
		float den = 1 + x * (0.498089 + x * (0.259956 + x * (0.0741939 + x * (0.0134601 + x * 0.000650605))));
		return x * x * (num / den);
	}
	else
	{
		// beyond this point the curve is mostly linear with a very slight offset
		// 3 fmas + 1 log
		// TODO: would it be better to use log2 directly and save one mult? can the compiler do this for us?
		return x - 0.25 * log(-5.68881 + x * (-19.7156 + 39.4779 * x));
	}
}

float FastASin(float x) {
	// Maximum error: 10^-3.35
	// TODO: consider replacing acosFast/asinFast which have approximation error 10^-2.0 (with one less fma, but also with a slightly sub-optimal constant)
	float a = saturate(abs(x));
	float r = (0.5 * PI) + a * (-0.207034 + 0.0531013 * a);
	r = (0.5 * PI) - r * sqrt(1 - a);
	return x >= 0 ? r : -r;
}

float Mp(float SinThetaI, float SinThetaO, float v)
{
	float rv = rcp(v);
	float CosThetaO2 = saturate(1 - SinThetaO * SinThetaO);
	float CosThetaI2 = saturate(1 - SinThetaI * SinThetaI);
	float CosProduct = sqrt(CosThetaO2 * CosThetaI2);
	// "Path tracing in Production" - Siggraph 2018 Course
	// "From Bricks to Bunnies - Adapting a Raytracer to New Requirements" - Luke Emrose
	// https://jo.dreggn.org/path-tracing-in-production/2018/course-notes.pdf
	// Simplified expression for better numerical stability - Eq(62)
	return exp(LogI0(CosProduct * rv) - rv * (SinThetaI * SinThetaO + 1) + log(rv) - log(1 - exp(-2 * rv)));
}

float TrimmedLogistic(float x, float InvS)
{
	// Logistic distribution, re-normalized to have bounds [-Pi,Pi]
	// The normalization constant below ensures we integrate to 1.0
	float e = exp(-abs(x * InvS));
	float a = exp(-PI * InvS); // normalization constant
	return e * InvS * (1 + a) / ((1 - a) * Pow2(1 + e));
}

float SampleTrimmedLogistic(float x, float s)
{
	float a = exp(-PI * rcp(s));
	float k = (1 - a) / (1 + a);
	float r = 2 * x - 1;
	return s * log((1 + r * k) / (1 - r * k));
}

float Phi(int p, float GammaO, float GammaT)
{
	return 2 * p * GammaT - 2 * GammaO + p * PI;
}

float Np(float phi, int p, float InvS, float GammaO, float GammaT) {
	// center around chosen angle
	float dphi = phi - Phi(p, GammaO, GammaT);
	// wrap to interval (-PI,PI)
	dphi /= 2 * PI;
	dphi -= round(dphi);
	dphi *= 2 * PI;
	return TrimmedLogistic(dphi, InvS);
}

// returns sine from cosine (or vice-versa)
float TrigInverse(float x)
{
	return sqrt(saturate(1 - x * x));
}

// precomputed terms for evaluating/sampling the hair bxdf
// TODO: can we somehow do this stuff only once instead of repeating it in eval and sample?
struct FHairData
{
	// trig that only depends on viewing direction and offset h
	float4 SinThetaO; // Viewing angles (adjusted by scale tilt per lobe)
	float GammaO;
	float GammaT;

	float  v0;    // roughness parameter for Mp
	float  v1;
	float  v2;
	float  v3;
	float  s;     // azimuthal roughness parameter for Np
	float  A0;    // attenuation term per lobe (first one is always white)
	float3 A1;
	float3 A2;
	float3 A3;
	float4 LobePdf;
	float3 LobeCdf; // Normalized sum to help select a lobe
};

FHairData PrepareHairData(float3 V, float3 Color, float LongitudinalRoughness, float AzimuthalRoughness, float h, float Specular)
{
	FHairData Result = (FHairData)0;
	// Longitudinal roughness
	float Bm = max(LongitudinalRoughness, 0.01); // avoid tiny values
	float Bm2 = Bm * Bm;
	float Bm4 = Bm2 * Bm2;
	float Bm8 = Bm4 * Bm4;
	float Bm10 = Bm8 * Bm2;
	float Bm20 = Bm10 * Bm10;
	// Chiang et al. - Eq(7)
	Result.v0 = Pow2(0.726 * Bm + 0.812 * Bm2 + 3.7 * Bm20);
	Result.v1 = 0.25 * Result.v0;
	Result.v2 = 4.00 * Result.v0;
	Result.v3 = Result.v2;

	// Azimuthal roughness
	const float SqrtPiOver8 = 0.626657069;
	float Bn = max(AzimuthalRoughness, 0.01);
	float Bn2 = Bn * Bn;
	float Bn4 = Bn2 * Bn2;
	float Bn8 = Bn4 * Bn4;
	float Bn11 = Bn8 * Bn2 * Bn;
	float Bn22 = Bn11 * Bn11;
	// Chiang et al. - Eq(8) + Eq(12)
	Result.s = SqrtPiOver8 * (0.265 * Bn + 1.194 * Bn2 + 5.372 * Bn22);

	// Specular to Eta
	float F0 = Specular * 0.08;
	float F90 = saturate(Specular * (50.0 * 0.08));
	float Eta = (1 + sqrt(F0)) / (1 - sqrt(F0));
	
	// setup main angles
	float SinThetaO = V.z;
	float CosThetaO = TrigInverse(SinThetaO);

	// TODO: should this angle be exposed as a parameter?
	// Using value from HairBsdf.ush for now
	float ScaleAngle = 0.035; // about 2 degrees
	float3 SinScale; // sin(ScaleAngle * 2^k) for k=0,1,2
	float3 CosScale; // cos(ScaleAngle * 2^k) for k=0,1,2
	SinScale.x = sin(ScaleAngle);
	CosScale.x = cos(ScaleAngle);
	SinScale.y = 2 * CosScale.x * SinScale.x;
	CosScale.y = Pow2(CosScale.x) - Pow2(SinScale.x);
	SinScale.z = 2 * CosScale.y * SinScale.y;
	CosScale.z = Pow2(CosScale.y) - Pow2(SinScale.y);

	// rotate by scale angle (different amount of each lobe)
	Result.SinThetaO.x = SinThetaO * CosScale.y - CosThetaO * SinScale.y; 	// R: -2*angle
	Result.SinThetaO.y = SinThetaO * CosScale.x + CosThetaO * SinScale.x;	// TT: +angle
	Result.SinThetaO.z = SinThetaO * CosScale.z + CosThetaO * SinScale.z;	// TRT: +4*angle
	Result.SinThetaO.w = SinThetaO;	// TR*T: 0 angle

	// Account for refraction through the cylinder
	float SinGammaT = h * CosThetaO * rsqrt(Pow2(Eta) - Pow2(SinThetaO));
	float CosGammaT2 = 1 - SinGammaT * SinGammaT;
	Result.GammaO = FastASin(h);
	Result.GammaT = FastASin(SinGammaT);

	float SinThetaT = SinThetaO / Eta;
	float CosThetaT2 = 1 - SinThetaT * SinThetaT;

	// Tinting factors through the hair
	float CosGammaO = sqrt(saturate(1 - h * h));
	float CosTheta = CosThetaO * CosGammaO;
	float F = lerp(F0, F90, Pow5(1 - CosTheta)); // Schlick approximation

	// Chiang et al. - Eq(9)
	float3 SigmaA = Pow2(log(max(Color, 1e-4)) / ((((((0.245 * Bn) + 5.574) * Bn - 10.73) * Bn + 2.532) * Bn - 0.215) * Bn + 5.969));
	
	// Zinke 2007 - Eq(20)
	float3 T = exp(-SigmaA * (2 * sqrt(CosGammaT2 / CosThetaT2)));

	// d'Eon et al. - Eq(12) + Eq(13)
	Result.A0 = F;
	Result.A1 = Pow2(1 - F) * T;
	Result.A2 = Result.A1 * F * T;
	// Chiang et al. - Eq(6)
	Result.A3 = Result.A2 * F * T * rcp(1.00001 - F * T); // avoid dividing by 0 when F*T=1

	Result.LobeCdf = LobeSelectionCdf(Result.A0, Result.A1, Result.A2, Result.A3);
	Result.LobePdf = LobeSelectionPdf(Result.LobeCdf);

	return Result;
}

FMaterialSample Hair_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample)
{
	const float3x3 HairBasis = GetHairBasis(GetHairTangent(Payload), V_World);

	const float3 V = mul(HairBasis, V_World); // TODO: simplify this since HairBasis is already sort of aligned with V ?
	const float h = 2 * Payload.GetHairPrimitiveUV().y - 1; // remap back to [-1,1]
	const float LongitudinalRoughness = Payload.GetHairLongitudinalRoughness();
	const float AzimuthalRoughness = lerp(1.0, 0.1, Payload.GetHairAzimuthalRoughness());
	const FHairData HairData = PrepareHairData(V, Payload.GetBaseColor(), LongitudinalRoughness, AzimuthalRoughness, h, Payload.GetHairSpecular());

	// 1) Decide which lobe to sample
	float u = RandSample.z;
	int p = 0;
	float v = 0, SinThetaOp = 0;
	if (u < HairData.LobeCdf.x)
	{
		u = RescaleRandomNumber(u, 0.0, HairData.LobeCdf.x);
		p = 0;
		v = HairData.v0;
		SinThetaOp = HairData.SinThetaO.x;
	}
	else if (u < HairData.LobeCdf.y)
	{
		u = RescaleRandomNumber(u, HairData.LobeCdf.x, HairData.LobeCdf.y);
		p = 1;
		v = HairData.v1;
		SinThetaOp = HairData.SinThetaO.y;
	}
	else if (u < HairData.LobeCdf.z)
	{
		u = RescaleRandomNumber(u, HairData.LobeCdf.y, HairData.LobeCdf.z);
		p = 2;
		v = HairData.v2;
		SinThetaOp = HairData.SinThetaO.z;
	}
	else
	{
		u = RescaleRandomNumber(u, HairData.LobeCdf.z, 1.0);
		p = 3;
		v = HairData.v3;
		SinThetaOp = HairData.SinThetaO.w;
	}

	// 2) Sample longitudinal angle
	const float CosTheta = 1 + v * log(RandSample.x + (1 - RandSample.x) * exp(-2 / v));
	const float CosPhi = cos(2 * PI * RandSample.y);
	const float SinThetaI = -CosTheta * SinThetaOp + sqrt((1 - CosTheta * CosTheta) * (1 - SinThetaOp * SinThetaOp)) * CosPhi;
	const float CosThetaI = TrigInverse(SinThetaI);

	// 3) Sample Azimuthal angle
	const float PhiI = (p < 3) ? Phi(p, HairData.GammaO, HairData.GammaT) + SampleTrimmedLogistic(u, HairData.s) : 2 * PI * u;

	const float3 L = float3(CosThetaI * sin(PhiI), CosThetaI * cos(PhiI), SinThetaI);

	// transform direction into world space
	const float3 L_World = mul(L, HairBasis);

	FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, sign(dot(Payload.WorldGeoNormal, L_World)), LongitudinalRoughness, PATHTRACER_SCATTER_SPECULAR);

	const float InvS = rcp(HairData.s);
	const float Pdf0 = Mp(SinThetaI, HairData.SinThetaO.x, HairData.v0) * Np(PhiI, 0, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf1 = Mp(SinThetaI, HairData.SinThetaO.y, HairData.v1) * Np(PhiI, 1, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf2 = Mp(SinThetaI, HairData.SinThetaO.z, HairData.v2) * Np(PhiI, 2, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf3 = Mp(SinThetaI, HairData.SinThetaO.w, HairData.v3) * (1 / (2 * PI)); // last lobe is isotropic

	Result.AddLobeWithMIS(HairData.A0, Pdf0, HairData.LobePdf.x);
	Result.AddLobeWithMIS(HairData.A1, Pdf1, HairData.LobePdf.y);
	Result.AddLobeWithMIS(HairData.A2, Pdf2, HairData.LobePdf.z);
	Result.AddLobeWithMIS(HairData.A3, Pdf3, HairData.LobePdf.w);

	return Result;
}

FMaterialEval Hair_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const float3x3 HairBasis = GetHairBasis(GetHairTangent(Payload), V_World);

	const float3 V = mul(HairBasis, V_World);
	const float3 L = mul(HairBasis, L_World);
	const float h = 2 * Payload.GetHairPrimitiveUV().y - 1; // remap back to [-1,1]
	const float LongitudinalRoughness = Payload.GetHairLongitudinalRoughness();
	const float AzimuthalRoughness = lerp(1.0, 0.1, Payload.GetHairAzimuthalRoughness());
	const FHairData HairData = PrepareHairData(V, Payload.GetBaseColor(), LongitudinalRoughness, AzimuthalRoughness, h, Payload.GetHairSpecular());

	// precompute data based of lighting direction
	const float SinThetaI = L.z;
	const float CosThetaI = TrigInverse(SinThetaI);
	const float PhiI = atan2(L.x, L.y); // NOTE: in our coordinate system, PhiO == 0, so we only need to account for L

	FMaterialEval Result = NullMaterialEval();

	const float InvS = rcp(HairData.s);
	const float Pdf0 = Mp(SinThetaI, HairData.SinThetaO.x, HairData.v0) * Np(PhiI, 0, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf1 = Mp(SinThetaI, HairData.SinThetaO.y, HairData.v1) * Np(PhiI, 1, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf2 = Mp(SinThetaI, HairData.SinThetaO.z, HairData.v2) * Np(PhiI, 2, InvS, HairData.GammaO, HairData.GammaT);
	const float Pdf3 = Mp(SinThetaI, HairData.SinThetaO.w, HairData.v3) * (1 / (2 * PI)); // last lobe is isotropic

	Result.AddLobeWithMIS(HairData.A0, Pdf0, HairData.LobePdf.x);
	Result.AddLobeWithMIS(HairData.A1, Pdf1, HairData.LobePdf.y);
	Result.AddLobeWithMIS(HairData.A2, Pdf2, HairData.LobePdf.z);
	Result.AddLobeWithMIS(HairData.A3, Pdf3, HairData.LobePdf.w);
	Result.Weight *= DiffuseSpecularScale.y;
	return Result;
}

#endif
