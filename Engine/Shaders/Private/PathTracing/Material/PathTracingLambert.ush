// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingLambert.usf: Lambertian BRDF sampling functions
===============================================================================================*/
#pragma once

FMaterialSample Lambert_SampleMaterial(
	FPathTracingPayload Payload,
	float3 RandSample)
{
	const float3 N_World = Payload.WorldNormal;
	const float3 BaseColor = Payload.GetBaseColor();

	float4 SampledValue = CosineSampleHemisphereWorld(RandSample.xy, N_World);

	float3 L_World = SampledValue.xyz;
#if SIMPLIFIED_MATERIAL_SHADER
	float ShadowTerminator = 1;
#else
	float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
#endif

	return CreateMaterialSample(L_World, Payload.BSDFOpacity * BaseColor * ShadowTerminator, SampledValue.w, 1.0, 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval Lambert_EvalMaterial(
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const float3 N_World = Payload.WorldNormal;
	const float3 BaseColor = Payload.GetBaseColor();

	float NoL = saturate(dot(N_World, L_World));
#if SIMPLIFIED_MATERIAL_SHADER
	float ShadowTerminator = 1;
#else
	float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
#endif

	return CreateMaterialEval(Payload.BSDFOpacity * BaseColor * ShadowTerminator * DiffuseSpecularScale.x, NoL / PI);
}

#if SIMPLIFIED_MATERIAL_SHADER

FMaterialSample Simplified_SampleMaterial(
	FPathTracingPayload Payload,
	float3 RandSample)
{
	float3 N_World = Payload.WorldNormal;

	float3 FrontColor = Payload.BaseColor;
	float3 BackColor = Payload.SubsurfaceColor;

	float TransmissionProb = LobeSelectionProb(BackColor, FrontColor);
	bool bDoTransmission = RandSample.z < TransmissionProb;
	float3 Weight = bDoTransmission ? BackColor / TransmissionProb : FrontColor / (1 - TransmissionProb);
	float Sign = bDoTransmission ? -1.0 : 1.0;
	float4 SampledValue = CosineSampleHemisphereWorld(RandSample.xy, Sign * N_World);
	float Pdf = SampledValue.w * (bDoTransmission ? TransmissionProb : 1.0 - TransmissionProb);

	Weight *= Payload.BSDFOpacity;
	return CreateMaterialSample(SampledValue.xyz, Weight, Pdf, Sign, 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval Simplified_EvalMaterial(
	float3 L_World,
	FPathTracingPayload Payload
)
{
	float3 N_World = Payload.WorldNormal;
	float NoL = dot(N_World, L_World);

	float3 FrontColor = Payload.BaseColor;
	float3 BackColor = Payload.SubsurfaceColor;
	float TransmissionProb = LobeSelectionProb(BackColor, FrontColor);

	float3 Weight = NoL < 0.0 ? BackColor : FrontColor;
	float Pdf = NoL < 0.0 ? TransmissionProb : 1.0 - TransmissionProb;
	if (Pdf > 0.0)
	{
		Weight /= Pdf;
	}
	Weight *= Payload.BSDFOpacity;
	return CreateMaterialEval(Weight, Pdf * saturate(abs(NoL)) / PI);
}

#endif // SIMPLIFIED_MATERIAL_SHADER
