// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingMaterialCommon.usf: Brdf utility functions
===============================================================================================*/

#pragma once

#include "/Engine/Private/MonteCarlo.ush"
#include "/Engine/Private/BRDF.ush"

float LobeColorToWeight(float3 C)
{
	// TODO: What is the best heuristic to use here?
	// Sum seems to perform slightly better sometimes, while max3 is better in others. Need to look at more cases to be sure
	// Also tried Luminance, but the result was very close to the plain sum
#if 1
	return C.x + C.y + C.z;
#else
	return max3(C.x, C.y, C.z);
#endif
}

// Given two lobes that will roughly contribute colors A and B to the total (estimated for example using directional albedo)
// return the probability of choosing lobe A
float LobeSelectionProb(float3 A, float3 B)
{
	const float Aw = LobeColorToWeight(A);
	const float Bw = LobeColorToWeight(B);
	// use MIS routine to robustly compute Aw / (Aw + Bw) even when both are 0, or overflow, etc ...
	return MISWeightBalanced(Aw, Bw);
}

// Given three lobes that will roughly contribute colors A, B and C to the total (estimated for example using directional albedo)
// return a CDF for choosing lobe A, B or C
// The individual probabilities of each lobe can be estimated using LobeSelectionPdf.
float2 LobeSelectionCdf(float3 A, float3 B, float3 C)
{
	const float Aw = LobeColorToWeight(A);
	const float Bw = LobeColorToWeight(B);
	const float Cw = LobeColorToWeight(C);

	// use MIS routine to robustly compute the ratios even when both are 0, or overflow, etc ...
	const float ProbAB = MISWeightBalanced(Aw, Bw); // A or B
	const float ProbLR = MISWeightBalanced(Aw + Bw, Cw); // AB or C

	// Now build CDF
	return float2(ProbAB * ProbLR, ProbLR);
}

// Same as above for choosing among 4 colored lobes
float3 LobeSelectionCdf(float3 A, float3 B, float3 C, float3 D)
{
	const float Aw = LobeColorToWeight(A);
	const float Bw = LobeColorToWeight(B);
	const float Cw = LobeColorToWeight(C);
	const float Dw = LobeColorToWeight(D);

	// use MIS routine to robustly compute the ratios even when both are 0, or overflow, etc ...
	const float ProbAB = MISWeightBalanced(Aw, Bw); // A or B
	const float ProbCD = MISWeightBalanced(Cw, Dw); // C or D
	const float ProbLR = MISWeightBalanced(Aw + Bw, Cw + Dw); // AB or CD

	// Now build CDF
	return float3(ProbAB * ProbLR, ProbLR, (ProbLR + ProbCD) - ProbLR * ProbCD);
}

// Given the CDF for a choice between three lobes (as returned by LobeSelectionCdf above), compute the Pdf for each individual lobe
float3 LobeSelectionPdf(float2 LobeCdf)
{
	return float3(LobeCdf.x, LobeCdf.y - LobeCdf.x, 1.0 - LobeCdf.y);
}

// Given the CDF for a choice between four lobes (as returned by LobeSelectionCdf above), compute the Pdf for each individual lobe
float4 LobeSelectionPdf(float3 LobeCdf)
{
	return float4(LobeCdf.x, LobeCdf.y - LobeCdf.x, LobeCdf.z - LobeCdf.y, 1.0 - LobeCdf.z);
}

// The following structs are used as return types for the Eval/Sample methods

#define PATHTRACER_SCATTER_CAMERA   0 // this is used to mark camera rays until we have actually scattered
#define PATHTRACER_SCATTER_DIFFUSE  1
#define PATHTRACER_SCATTER_SPECULAR 2
#define PATHTRACER_SCATTER_REFRACT  3
#define PATHTRACER_SCATTER_VOLUME   4

struct FMaterialSample
{
	float3 Direction;
	float3 Weight;
	float Pdf;
	float PositionBiasSign;
	float Roughness;
	int   ScatterType;

	void AddLobeWithMIS(float3 LobeWeight, float LobePdf, float LobeProb)
	{
		::AddLobeWithMIS(Weight, Pdf, LobeWeight, LobePdf, LobeProb);
	}
};

FMaterialSample NullMaterialSample()
{
	// return a zero initialized sample, for cases where we couldn't sample the material (such as rays below the horizon, etc..)
	return (FMaterialSample)0;
}

FMaterialSample CreateMaterialSample(float3 Direction, float3 Weight, float Pdf, float PositionBiasSign, float Roughness, int ScatterType)
{
	FMaterialSample Result;
	Result.Direction = Direction;
	Result.Weight = Weight;
	Result.Pdf = Pdf;
	Result.PositionBiasSign = PositionBiasSign;
	Result.Roughness = Roughness;
	Result.ScatterType = ScatterType;
	return Result;
}

struct FMaterialEval
{
	float3 Weight;
	float Pdf;

	void AddLobeWithMIS(float3 LobeWeight, float LobePdf, float LobeProb)
	{
		::AddLobeWithMIS(Weight, Pdf, LobeWeight, LobePdf, LobeProb);
	}
};

FMaterialEval NullMaterialEval()
{
	return (FMaterialEval)0;
}

FMaterialEval CreateMaterialEval(float3 Weight, float Pdf)
{
	FMaterialEval Result;
	Result.Weight = Weight;
	Result.Pdf = Pdf;
	return Result;
}



#define PATH_TRACER_SHADOW_TERMINATOR   1 /// 0: off 1: imageworks 2: disney 3: dreamworks (see paper references below)

float ShadowTerminatorTerm(float3 L, float3 N, float3 Ns)
{
	// #jira UE-121401
#if PATH_TRACER_SHADOW_TERMINATOR == 1
	// Imageworks terminator softening:
	// "A Microfacet-Based Shadowing Function to Solve the Bump Terminator Problem"
	// Alejandro Conty Estevez, Pascal Lecocq, and Clifford Stein.
	// Ray Tracing Gems(2019), 149 - 158.
	// http://www.aconty.com/pdf/bump-terminator-nvidia2019.pdf
	const float Epsilon = 1e-6; // avoid division by 0
	const float CosD = saturate(abs(dot(Ns, N)));
	const float Tan2D = (1.0 - CosD * CosD) / (CosD * CosD + Epsilon);
	const float Alpha2 = saturate(0.125 * Tan2D);
	const float CosI = saturate(dot(Ns, L));
	const float Tan2I = (1.0f - CosI * CosI) / (CosI * CosI + Epsilon);
	return CosI > 0 ? 2.0f / (1.0f + sqrt(1.0f + Alpha2 * Tan2I)) : 0.0;
#elif PATH_TRACER_SHADOW_TERMINATOR == 2
	// Disney terminator softening:
	// "Taming the Shadow Terminator"
	// Matt Jen-Yuan Chiang, Yining Karl Li, and Brent Burley
	// SIGGRAPH 2019 Talks
	// https://www.yiningkarlli.com/projects/shadowterminator.html
	const float NoL = saturate(dot(N, L));
	const float NgoL = saturate(dot(Ns, L));
	const float NgoN = saturate(dot(Ns, N));
	const float G = saturate(NgoL / (NoL * NgoN + 1e-6));
	return G + G * (G - G * G); // smooth
#elif PATH_TRACER_SHADOW_TERMINATOR == 3
	// Dreamworks terminator softening:
	// "Predictable and Targeted Softening of the Shadow Terminator"
	// Priyamvad Deshmukh, Brian Green
	// SIGGRAPH 2020 Talks
	// https://research.dreamworks.com/wp-content/uploads/2020/08/talk_shadow_terminator.pdf
	const float CosD = saturate(abs(dot(Ns, N)));
	const float d = lerp(0.15, 0.05, CosD);
	const float t = saturate(dot(Ns, L) / d);
	return t * t * (3.0 - 2.0 * t);
#else
	// no special handling of the terminator, don't do anything
	return 1.0;
#endif
}

float3 GetPathTracingDiffuseModel(float3 DiffuseColor, float Roughness, float NoV, float NoL, float VoH, float NoH)
{
#if MATERIAL_ROUGHDIFFUSE
	// Cancel out the factor of PI because the path tracer wants the sample weight
	return PI * Diffuse_GGX_Rough(DiffuseColor, Roughness, NoV, NoL, VoH, NoH, 1.0f);
#else
	return DiffuseColor;
#endif
}

// Given two diffuse colors as found on either SHADINGMODELID_TWOSIDED_FOLIAGE or SHADINGMODELID_SUBSURFACE, balance them out to ensure they don't reflect more than 100% energy
void AdjustPathTracingTwoSidedColorBalance(inout float3 FrontColor, inout float3 BackColor)
{
	float3 Sum = FrontColor + BackColor;
	float S = max(Sum.x, max(Sum.y, Sum.z));
	if (S > 1)
	{
		// nudge the material back to something physically plausible
		// NOTE: we ignore spec here since it should be accounted for by the brdf model itself
		FrontColor /= S;
		BackColor /= S;
	}
}