// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingMaterialSampling.usf: Material sampling functions
===============================================================================================*/

#pragma once

#include "PathTracingMaterialCommon.ush"
#include "PathTracingFresnel.ush"
#include "PathTracingLambert.ush"
#if !SIMPLIFIED_MATERIAL_SHADER
#include "PathTracingGlossy.ush"
#include "PathTracingSubstrate.ush"
#include "PathTracingSubstrateFoliage.ush"
#include "PathTracingSubstrateSolidGlass.ush"
#include "PathTracingSubstrateThinGlass.ush"
#include "PathTracingEye.ush"
#include "PathTracingHair.ush"
#include "PathTracingMedium.ush"
#endif // !SIMPLIFIED_MATERIAL_SHADER

float3 EstimateMaterialAlbedo(FPathTracingPayload Payload)
{
	// TODO: make this more accurate by estimating the directional albedo?
	float3 Albedo = 0;
	switch (Payload.PathTracingShadingModelID)
	{
#if SIMPLIFIED_MATERIAL_SHADER
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
			Albedo = Payload.BaseColor + Payload.SubsurfaceColor;
			break;
#else
		case PATH_TRACING_SHADINGMODELID_BASIC:
		{
			Albedo = Payload.DiffuseColor + Payload.SubsurfaceColor + Payload.SpecularColor + Payload.FuzzAmount * Payload.FuzzColor;
			Albedo *= Payload.GetMaxLobeWeight();
			break;
		}
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS:
		{
			Albedo = 1.0 + Payload.FuzzAmount * Payload.FuzzColor;
			Albedo *= Payload.GetMaxLobeWeight();
			break;
		}
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
		{
			Albedo = Payload.DiffuseColor + Payload.GetFoliageTransmissionColor() + Payload.SpecularColor + Payload.FuzzAmount * Payload.FuzzColor;
			Albedo *= Payload.GetMaxLobeWeight();
			break;
		}
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS:
		{
			const float F0 = F0RGBToF0(Payload.SpecularColor);
			Albedo = Payload.HasRefraction() ? lerp(Payload.GetTransmittanceColor(), 1.0, F0) : F0;
			Albedo += Payload.DiffuseColor + Payload.FuzzAmount * Payload.FuzzColor;
			Albedo *= Payload.GetMaxLobeWeight();
			break;
		}
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
		{
			Albedo = Payload.DiffuseColor + Payload.SubsurfaceColor + Payload.SpecularColor + Payload.RoughnessData.z * CLEAR_COAT_F0 + Payload.FuzzAmount * Payload.FuzzColor;
			Albedo *= Payload.GetMaxLobeWeight();
			break;
		}
		case PATH_TRACING_SHADINGMODELID_EYE:
			Albedo = Payload.DiffuseColor + Payload.SubsurfaceColor + Payload.SpecularColor;
			break;
		case PATH_TRACING_SHADINGMODELID_HAIR:
		// SUBSTRATE_TODO: Add more cases here?
#endif
		default:
			Albedo = Payload.GetBaseColor();
			break;
	}
	Albedo *= Payload.BSDFOpacity;

	return Albedo;
}

#if !SIMPLIFIED_MATERIAL_SHADER

#define ROUGHNESS_WEIGHT(r)		DenoiserRoughnessWeight(r, PathRoughness)

void AccumulateAlbedoNormal(inout FPathTracingPayload Payload, float3 PathThroughput, float PathRoughness, inout float3 Albedo, inout float3 Normal)
{
	float AOVWeight = DenoiserAOVWeight(PathRoughness);
	if (AOVWeight == 0)
	{
		// no contribution
		return;
	}
	PathThroughput *= AOVWeight;
	PathThroughput *= Payload.WeightV * Payload.BSDFOpacity * lerp(1.0, Payload.TransmittanceN, Payload.CoverageAboveAlongN);
	float NormThroughput = Luminance(PathThroughput);
	// figure out how much the specular lobes should contribute to the total (relative to the current PathRoughness)
	// if we are above the MinRoughness, use the PathRoughness as are lower bound instead so that multiple bounces don't accumulate too much albedo
	switch (Payload.PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_BASIC:
		{
			const float Roughness0 = Payload.RoughnessData.x;
			const float Roughness1 = Payload.RoughnessData.y;
			const float SpecBlend = Payload.RoughnessData.z;
			const float RoughnessWeight = lerp(ROUGHNESS_WEIGHT(Roughness0), ROUGHNESS_WEIGHT(Roughness1), SpecBlend);
			Albedo += PathThroughput * (Payload.DiffuseColor + Payload.SubsurfaceColor + Payload.SpecularColor * RoughnessWeight + Payload.FuzzAmount * Payload.FuzzColor);
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS:
		{
			const float Roughness0 = Payload.RoughnessData.x;
			const float Roughness1 = Payload.RoughnessData.y;
			const float SpecBlend = Payload.RoughnessData.z;
			const float RoughnessWeight = lerp(ROUGHNESS_WEIGHT(Roughness0), ROUGHNESS_WEIGHT(Roughness1), SpecBlend);
			float GlassAlbedo = 1.0;
			Albedo += PathThroughput * (GlassAlbedo * RoughnessWeight + Payload.FuzzAmount * Payload.FuzzColor);
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
		{
			const float Roughness0 = Payload.RoughnessData.x;
			const float Roughness1 = Payload.RoughnessData.y;
			const float SpecBlend = Payload.RoughnessData.z;
			const float RoughnessWeight = lerp(ROUGHNESS_WEIGHT(Roughness0), ROUGHNESS_WEIGHT(Roughness1), SpecBlend);
			Albedo += PathThroughput * (Payload.DiffuseColor + Payload.GetFoliageTransmissionColor() + Payload.SpecularColor * RoughnessWeight + Payload.FuzzAmount * Payload.FuzzColor);
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS:
		{
			const float F0 = F0RGBToF0(Payload.SpecularColor);
			const float3 GlassAlbedo = Payload.HasRefraction() ? lerp(Payload.GetTransmittanceColor(), 1.0, F0) : F0;
			const float Roughness0 = Payload.RoughnessData.x;
			const float Roughness1 = Payload.RoughnessData.y;
			const float SpecBlend = Payload.RoughnessData.z;
			const float RoughnessWeight = lerp(ROUGHNESS_WEIGHT(Roughness0), ROUGHNESS_WEIGHT(Roughness1), SpecBlend);
			Albedo += PathThroughput * (RoughnessWeight * GlassAlbedo + Payload.DiffuseColor + Payload.FuzzAmount * Payload.FuzzColor);
			Normal += NormThroughput * RoughnessWeight * Payload.WorldNormal;

		}
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
		{
			const float Roughness0 = Payload.RoughnessData.x;
			const float Roughness1 = Payload.RoughnessData.y;
			const float SpecBlend = Payload.RoughnessData.z;
			const float Roughness0Weight = ROUGHNESS_WEIGHT(Roughness0);
			const float Roughness1Weight = SpecBlend * ROUGHNESS_WEIGHT(Roughness1);
			Albedo += PathThroughput * (Payload.DiffuseColor + Roughness0Weight * Payload.SpecularColor + Roughness1Weight * CLEAR_COAT_F0 + Payload.FuzzAmount * Payload.FuzzColor);
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
		case PATH_TRACING_SHADINGMODELID_EYE:
		{
			const float RoughnessWeight = ROUGHNESS_WEIGHT(Payload.GetEyeRoughness());
			Albedo += PathThroughput * (Payload.DiffuseColor + RoughnessWeight * Payload.SpecularColor + Payload.SubsurfaceColor);
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
		default:
		{
			Albedo += PathThroughput * Payload.GetBaseColor();
			Normal += NormThroughput * Payload.WorldNormal;
			break;
		}
	}
}
#undef ROUGHNESS_WEIGHT

void AccumulateAlbedo(float3 SigmaS, float3 PathThroughput, float PathRoughness, inout float3 Albedo)
{
	PathThroughput *= DenoiserAOVWeight(PathRoughness);
	// Volume acts as diffuse scattering, so make it contribute fully (like DiffuseColor above)
	Albedo += SigmaS * PathThroughput;
}

#endif // SIMPLIFIED_MATERIAL_SHADER

// returns an average roughness for the whole material (for the path roughness of shadow rays from this material)
float GetAverageRoughness(FPathTracingPayload Payload)
{
#if SIMPLIFIED_MATERIAL_SHADER
	return 1.0;
#else
	float AvgRoughness = Payload.RoughnessData.x;
	switch (Payload.PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_BASIC:
		{
			AvgRoughness = max(AvgRoughness, Payload.RoughnessData.y);
			// Only non-glass slabs contain diffuse
			float DiffuseFraction = Payload.HasRefraction() ? 1.0 : F0RGBToMetallic(Payload.SpecularColor);
			AvgRoughness = lerp(1.0, AvgRoughness, DiffuseFraction);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS:
		{
			// NOTE: this case does not contain a diffuse lobe yet
			AvgRoughness = max(AvgRoughness, Payload.RoughnessData.y);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS:
		{
			AvgRoughness = max(AvgRoughness, Payload.RoughnessData.y);
			float DiffuseFraction = LobeSelectionProb(Payload.DiffuseColor + Payload.FuzzAmount * Payload.FuzzColor, Payload.SpecularColor);
			AvgRoughness = lerp(1.0, AvgRoughness, DiffuseFraction);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
		{
			AvgRoughness = max(AvgRoughness, Payload.RoughnessData.y);
			float DiffuseFraction = F0RGBToMetallic(Payload.SpecularColor + Payload.RoughnessData.z * CLEAR_COAT_F0);
			AvgRoughness = lerp(1.0, AvgRoughness, DiffuseFraction);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_EYE:
		{
			float DiffuseFraction = F0RGBToMetallic(Payload.SpecularColor);
			AvgRoughness = lerp(1.0, AvgRoughness, DiffuseFraction);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_MEDIUM:
			return 1;
		// SUBSTRATE_TODO: do we need special cases for the other slab types?
	}
	return AvgRoughness;
#endif
}

void AdjustDiffuseSSSContribution(inout float3 DiffColor, inout float3 SSSColor, inout float3 Radius)
{
 	// Below this radius, lerp to diffuse so we can avoid numerical issues in the random walk itself which lead to darkening
	const float MinRadius = 0.02;
	float3 Blend = saturate(Radius / MinRadius);
	DiffColor += SSSColor * (1 - Blend);
	SSSColor *= Blend;
	Radius = max(Radius, MinRadius);
}

#if !SIMPLIFIED_MATERIAL_SHADER
void AdjustPayloadAfterUnpack(inout FPathTracingPayload HitPayload, bool bApplyOverrides)
{
	// This function can also do a bit of per-material post-processing to simplify the rest of the code
	if (bApplyOverrides)
	{
		HitPayload.DiffuseColor  = HitPayload.DiffuseColor  * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
		HitPayload.SpecularColor = HitPayload.SpecularColor * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.xyz;
	}

	switch (HitPayload.PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_BASIC:
		{
			// NOTE: if radius is 0 the sss color will get flipped back into diffuse
			HitPayload.SubsurfaceColor = HitPayload.DiffuseColor;
			HitPayload.DiffuseColor = 0;
			AdjustDiffuseSSSContribution(HitPayload.DiffuseColor, HitPayload.SubsurfaceColor, HitPayload.MeanFreePath);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
		{
			float3 TransmissionColor = HitPayload.GetFoliageTransmissionColor();
			if (bApplyOverrides)
			{
				TransmissionColor = TransmissionColor * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
			}
			AdjustPathTracingTwoSidedColorBalance(HitPayload.DiffuseColor, TransmissionColor);
			HitPayload.SetFoliageTransmissionColor(TransmissionColor);
			break;
		}
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
		{
			if (bApplyOverrides)
			{
				HitPayload.RoughnessData.z = HitPayload.RoughnessData.z * View.SpecularOverrideParameter.w + View.SpecularOverrideParameter.x;
			}
			break;
		}
		case PATH_TRACING_SHADINGMODELID_EYE:
		{
			// eye still needs a diffuse component for the iris center
			float IrisMask = HitPayload.GetEyeIrisMask();
			HitPayload.SubsurfaceColor = HitPayload.DiffuseColor - IrisMask * HitPayload.DiffuseColor;
			HitPayload.DiffuseColor *= IrisMask;
			AdjustDiffuseSSSContribution(HitPayload.DiffuseColor, HitPayload.SubsurfaceColor, HitPayload.MeanFreePath);
			break;
		}
	}
}

struct FSSSRandomWalkInfo
{
	float3 Color;
	float3 Radius;
	float3 Weight;  // multiplier for the path throughput to take into account energy conservation from specular above the SSS
	float  Prob;    // probability of choosing to perform the random walk
	float  G;
};


FSSSRandomWalkInfo GetMaterialSSSInfo(FPathTracingPayload Payload, float3 V_World)
{
	FSSSRandomWalkInfo Result = (FSSSRandomWalkInfo)0;
	if (any(Payload.MeanFreePath > 0.0))
	{
		Result.Color = Payload.SubsurfaceColor;
		Result.Radius = Payload.MeanFreePath;
		Result.G = Payload.PhaseG;
		if (Payload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_BASIC)
		{
			FSubstrateSingleSlabData Data = (FSubstrateSingleSlabData)0;
			Data.PrepareBSDF(V_World, Payload, true);
			Result.Weight = Payload.WeightV * Payload.BSDFOpacity * Data.DiffuseWeight;
			Result.Prob = Data.LobePdf.x;
		}
		else if (Payload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE)
		{
			const float NoV = saturate(dot(Payload.WorldNormal, V_World));
			const float3 SpecE = ComputeGGXSpecEnergyTermsRGB(Payload.GetEyeRoughness(), NoV, Payload.SpecularColor).E;
			const float DiffWeight = (1 - Luminance(SpecE));
			const float3 SSSAlbedo = DiffWeight * Payload.SubsurfaceColor;
			const float3 DiffAlbedo = DiffWeight * Payload.DiffuseColor;
			const float3 SpecAlbedo = SpecE;
			Result.Prob = LobeSelectionProb(SSSAlbedo, DiffAlbedo + SpecAlbedo);
			Result.Weight = Payload.BSDFOpacity * DiffWeight;
		}
	}
	return Result;
}

void RemoveMaterialSSS(inout FPathTracingPayload Payload)
{
	// Tweak the payload to remove subsurface scattering without chaning the overall color
	// This function is free to assume this material contains SSS
	Payload.DiffuseColor += Payload.SubsurfaceColor;
	Payload.DiffuseColor = saturate(Payload.DiffuseColor);
	Payload.SubsurfaceColor = 0.0;
}

#endif // SIMPLIFIED_MATERIAL_SHADER


FMaterialSample SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
#if SIMPLIFIED_MATERIAL_SHADER
	return Simplified_SampleMaterial(Payload, RandSample);
#else
	switch (Payload.PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_VOLUMETRIC:
			return Volumetric_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_BASIC:
			return Substrate_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS:
			return SubstrateSolidGlass_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
			return SubstrateFoliage_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS:
			return SubstrateThinGlass_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
			return SubstrateClearCoat_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_EYE:
			return Eye_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_HAIR:
			return Hair_SampleMaterial(V_World, Payload, RandSample);
		case PATH_TRACING_SHADINGMODELID_MEDIUM:
			return Medium_SampleMaterial(V_World, Payload, RandSample);
		default:
			return Lambert_SampleMaterial(Payload, RandSample);
	}
	return NullMaterialSample();
#endif // SIMPLIFIED_MATERIAL_SHADER
}

FMaterialEval EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
#if SIMPLIFIED_MATERIAL_SHADER
	return Simplified_EvalMaterial(L_World, Payload);
#else
	switch (Payload.PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_VOLUMETRIC:
			return Volumetric_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_BASIC:
			return Substrate_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS:
			return SubstrateSolidGlass_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_FOLIAGE:
			return SubstrateFoliage_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS:
			return SubstrateThinGlass_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT:
			return SubstrateClearCoat_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_EYE:
			return Eye_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_HAIR:
			return Hair_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		case PATH_TRACING_SHADINGMODELID_MEDIUM:
			return Medium_EvalMaterial(V_World, L_World, Payload, DiffuseSpecularScale);
		default:
			return Lambert_EvalMaterial(L_World, Payload, DiffuseSpecularScale);
	}
	return NullMaterialEval();
#endif // SIMPLIFIED_MATERIAL_SHADER
}
