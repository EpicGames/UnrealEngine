// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingMedium.usf: Volumetric phase function
 * To support atmosphere, we represent a blend of two components: rayleigh scattering and an HG lobe.
 * To support volumetric clouds we also include a dual HG lobe that we track separately to simplify handling cases where multiple volumes overlap.
 * See FVolumeShadedResult for the logic around how this struct is tracked during volume tracing.
   
===============================================================================================*/
#pragma once

#include "../../ParticipatingMediaCommon.ush"

#define USE_UNIFORM_PHASE_FUNCTION 0

FMaterialSample Medium_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample)
{
	const float3 RayleighWeight = Payload.GetBaseColor();
	const float3 HG0 = Payload.GetHGWeight();
	const float G0 = Payload.GetHGPhaseG();
	const float3 DualHGPhaseData = Payload.GetDualHGPhaseData();
	const float3 HG1 = Payload.GetDualHGWeight() - Payload.GetDualHGWeight() * DualHGPhaseData.z;
	const float3 HG2 = Payload.GetDualHGWeight() * DualHGPhaseData.z;
	const float G1 = DualHGPhaseData.x;
	const float G2 = DualHGPhaseData.y;
	const float Gb = DualHGPhaseData.z;


#if USE_UNIFORM_PHASE_FUNCTION
	const float4 Result = UniformSampleSphere(RandSample.xy);
	// uniform scattering
	return CreateMaterialSample(Result.xyz, RayleighWeight + HG0 + HG1 + HG2, Result.w, 1.0, 1.0, PATHTRACER_SCATTER_VOLUME);
#else
	const float3 LobeCdf = LobeSelectionCdf(RayleighWeight, HG0, HG1, HG2);
	const float4 LobePdf = LobeSelectionPdf(LobeCdf);

	float CosTheta = 0.0; 
	if (RandSample.x < LobeCdf.x)
	{
		CosTheta = RayleighPhaseInvertCdf(RescaleRandomNumber(RandSample.x, 0.0, LobeCdf.x));
	}
	else
	{
		float G, R;
		if (RandSample.x < LobeCdf.y)
		{
			R = RescaleRandomNumber(RandSample.x, LobeCdf.x, LobeCdf.y);
			G = G0;
		}
		else if (RandSample.x < LobeCdf.z)
		{
			R = RescaleRandomNumber(RandSample.x, LobeCdf.y, LobeCdf.z);
			G = G1;
		}
		else
		{
			R = RescaleRandomNumber(RandSample.x, LobeCdf.z, 1.0);
			G = G2;
		}
		CosTheta = HenyeyGreensteinPhaseInvertCDF(R, G);
	}
	const float SinTheta = sqrt(saturate(1.0 - CosTheta * CosTheta));
	const float Phi = RandSample.y * (2.0 * PI);

	const float3 L_World = TangentToWorld(float3(cos(Phi) * SinTheta, sin(Phi) * SinTheta, CosTheta), -V_World);

	FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, 1.0, 1.0, PATHTRACER_SCATTER_VOLUME);
	Result.AddLobeWithMIS(RayleighWeight, RayleighPhase(CosTheta)            , LobePdf.x);
	Result.AddLobeWithMIS(HG0           , HenyeyGreensteinPhase(G0, CosTheta), LobePdf.y);
	Result.AddLobeWithMIS(HG1           , HenyeyGreensteinPhase(G1, CosTheta), LobePdf.z);
	Result.AddLobeWithMIS(HG2           , HenyeyGreensteinPhase(G2, CosTheta), LobePdf.w);
	return Result;
#endif
}

FMaterialEval Medium_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const float3 RayleighWeight = Payload.GetBaseColor();
	const float3 HG0 = Payload.GetHGWeight();
	const float G0 = Payload.GetHGPhaseG();
	const float3 DualHGPhaseData = Payload.GetDualHGPhaseData();
	const float3 HG1 = Payload.GetDualHGWeight() - Payload.GetDualHGWeight() * DualHGPhaseData.z;
	const float3 HG2 = Payload.GetDualHGWeight() * DualHGPhaseData.z;
	const float G1 = DualHGPhaseData.x;
	const float G2 = DualHGPhaseData.y;
	const float Gb = DualHGPhaseData.z;

#if USE_UNIFORM_PHASE_FUNCTION
	// simple omni-directional evaluation
	return CreateMaterialEval((RayleighWeight + HG0 + HG1 + HG2) * DiffuseSpecularScale.x, 1.0 / (4 * PI));
#else
	const float3 LobeCdf = LobeSelectionCdf(RayleighWeight, HG0, HG1, HG2);
	const float4 LobePdf = LobeSelectionPdf(LobeCdf);

	const float CosTheta = dot(V_World, L_World);
	FMaterialEval Result = NullMaterialEval();
	Result.AddLobeWithMIS(RayleighWeight, RayleighPhase(CosTheta)            , LobePdf.x);
	Result.AddLobeWithMIS(HG0           , HenyeyGreensteinPhase(G0, CosTheta), LobePdf.y);
	Result.AddLobeWithMIS(HG1           , HenyeyGreensteinPhase(G1, CosTheta), LobePdf.z);
	Result.AddLobeWithMIS(HG2           , HenyeyGreensteinPhase(G2, CosTheta), LobePdf.w);
	Result.Weight *= DiffuseSpecularScale.x;
	return Result;
#endif
}



FMaterialSample Volumetric_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample)
{

	const float Directionality = Payload.Anisotropy + 1;
	// Exact importance sampling of the Pdf below
	const float CosTheta = 
		Directionality > 1 ? (-1.0 + Directionality * 0.5 + (2 - 0.5 * Directionality) * sqrt(RandSample.x)) : (
		Directionality > 0 ? (-1.0 + sqrt(Pow4(1 - Directionality) + (4.0 - Directionality) * Directionality * (2 + Directionality * Directionality) * RandSample.x)) * rcp(2 * Directionality) :
			-1.0 + 2.0 * RandSample.x
	);
	const float SinTheta = sqrt(saturate(1.0 - CosTheta * CosTheta));
	const float Phi = RandSample.y * (2.0 * PI);

	const float3 L_World = TangentToWorld(float3(cos(Phi) * SinTheta, sin(Phi) * SinTheta, CosTheta), Payload.WorldNormal);

	const float3 Color = Payload.GetBaseColor() * Payload.BSDFOpacity;
	const float Pdf = Directionality > 1
		? ((4 - 2 * Directionality) + 4 * CosTheta) / (PI * Pow2(Directionality - 4))
		: (1 + (2 * Directionality) * CosTheta) / (((2 * PI) - (0.5 * PI) * Directionality) * (2 + Directionality * Directionality));
	return CreateMaterialSample(L_World, Color, max(Pdf, 0), sign(CosTheta), 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval Volumetric_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	const float Directionality = Payload.Anisotropy + 1;
	const float3 Color = Payload.GetBaseColor() * Payload.BSDFOpacity;
	const float CosTheta = dot(L_World, Payload.WorldNormal);
	// The PDF below has the following properties:
	// When Directionality == 0: uniform sampling of the sphere (all cosines from -1 to 1 are equaly likely)
	// When Directionality == 1: sample cosine linearly -1/2 to +1 
	// When Directionality == 2: sample cosine linearly from 0 to +1 (regular cosine lobe)
	// In between these values, the PDF changes smoothly and is always a linear function, so the CDF is quadratic and can be directly inverted (see above)
	const float Pdf = Directionality > 1
		? ((4 - 2 * Directionality) + 4 * CosTheta) / (PI * Pow2(Directionality - 4))
		: (1 + (2 * Directionality) * CosTheta) / (((2 * PI) - (0.5 * PI) * Directionality) * (2 + Directionality * Directionality));
	return CreateMaterialEval(Color * DiffuseSpecularScale.x, max(Pdf, 0));
}
