// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingLambert.usf: Lambertian BRDF sampling functions 
===============================================================================================*/
#pragma once

#if 1 // Cosine-weighted concentric sampling with optional guiding

#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
	#include "FirstBounceRayGuidingCommon.ush"
#endif

float2 InverseConcentricMapping(float2 p)
{
	// Handle degeneracy at the origin
	if (p.x == 0 && p.y == 0) return float2(0, 0);

	float r = sqrt(p.x * p.x + p.y * p.y);
	float theta = atan2(p.y, p.x);
	if (theta < -PI/4) theta += 2*PI;

	float a, b;
	if (theta < PI / 4) // region 1
	{
		a = r;
		b = theta * a / (PI/4);
	}
	else if (theta < 3*PI/4) // region 2
	{
		b = r;
		a = -(theta - PI/2) * b / (PI/4);
	}
	else if (theta < 5*PI/4) // region 3
	{
		a = -r;
		b = (theta - PI) * a / (PI/4);
	}
	else // region 4
	{
		b = -r;
		a = -(theta - 3*PI/2) * b / (PI/4);
	}

	float x = (a + 1) / 2;
	float y = (b + 1) / 2;

	return float2(x, y);
}

FMaterialSample RadianceProbe_SampleMaterial(
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	float PdfAdjust = 1.0f; 

	float2 SamplePoint = RandSample.yx;

#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
#if USE_FIRST_BOUNCE_RAY_GUIDING
	uint2 BatchedLaunchIndex = DispatchRaysIndex().xy;

	uint2 LaunchIndex = uint2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;

	int2 ClusterPosition = clamp((int2)LaunchIndex - int2(2, 2), int2(0, 0), int2(63, 63)) / TEXEL_CLUSTER_SIZE;
	
	int MinRenderPassIndex = NumRayGuidingTrialSamples;

#if USE_IRRADIANCE_CACHING
	MinRenderPassIndex += GetIrradianceCachingQuality();
#endif
	
	if (BatchedTiles[TileIndex].RenderPassIndex >= MinRenderPassIndex)
	{
		float LastRowPrefixSum = 0;
		float RowPrefixSum = 0;
		
		int Y = 0;
		
		UNROLL
		for (int y = 0; y < DIRECTIONAL_BINS_ONE_DIM; y++)
		{
			int2 WritePos = int2(y % 4, y / 4);
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM / 4 + WritePos;
			
			LastRowPrefixSum = RowPrefixSum;
			RowPrefixSum = RayGuidingCDFY[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE / 4 + FinalPosition];
			
			if (RowPrefixSum > SamplePoint.y)
			{
				SamplePoint.y = (y + (SamplePoint.y - LastRowPrefixSum) / (RowPrefixSum - LastRowPrefixSum)) / DIRECTIONAL_BINS_ONE_DIM;
				
				PdfAdjust *= (RowPrefixSum - LastRowPrefixSum) * DIRECTIONAL_BINS_ONE_DIM;
				
				Y = y;
				
				break;
			}
		}
		
		float LastPrefixSum = 0;
		float PrefixSum = 0;
		
		UNROLL
		for (int x = 0; x < DIRECTIONAL_BINS_ONE_DIM; x++)
		{
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + int2(x, Y);
			LastPrefixSum = PrefixSum;
			PrefixSum = RayGuidingCDFX[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition];
			
			if (PrefixSum > SamplePoint.x)
			{
				SamplePoint.x = (x + (SamplePoint.x - LastPrefixSum) / (PrefixSum - LastPrefixSum)) / DIRECTIONAL_BINS_ONE_DIM;									
				PdfAdjust *= (PrefixSum - LastPrefixSum) * DIRECTIONAL_BINS_ONE_DIM;
				break;
			}
		}
	}
#endif
#endif

	float3 N_World = Payload.WorldNormal;

#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
	// Use cosine hemisphere sampling for surface lightmaps
	float4 SampledValue = CosineSampleHemisphereConcentric(SamplePoint);
#else
	// Use uniform hemisphere sampling for volumetric lightmaps (the two hemispheres are meant to be merged)
	float4 SampledValue = UniformSampleHemisphere(SamplePoint);
#endif
	float3 OutDirection = TangentToWorld(SampledValue.xyz, N_World);
	
	float OutPdf = SampledValue.w * PdfAdjust;
	// As the name suggests, radiance probe has a combined throughput (Pdf * Weight) == 1
	float OutWeight = (OutPdf > 0.0f) ? (1.0f / OutPdf) : 0;

	return CreateMaterialSample(OutDirection, OutWeight, OutPdf, 1.0, 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval RadianceProbe_EvalMaterial(
	float3 L_World,
	FPathTracingPayload Payload
)
{
	float3 N_World = Payload.WorldNormal;
	
	float PdfAdjust = 1.0f;
	
#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG 
#if USE_FIRST_BOUNCE_RAY_GUIDING
	float3 TangentDirection = WorldToTangent(L_World, N_World);
	float2 PrimarySample = InverseConcentricMapping(TangentDirection.xy);
	
	uint2 BatchedLaunchIndex = DispatchRaysIndex().xy;

	uint2 LaunchIndex = uint2(BatchedLaunchIndex.x % GPreviewLightmapPhysicalTileSize, BatchedLaunchIndex.y);
	int TileIndex = BatchedLaunchIndex.x / GPreviewLightmapPhysicalTileSize;

	int2 ClusterPosition = clamp((int2)LaunchIndex - int2(2, 2), int2(0, 0), int2(63, 63)) / TEXEL_CLUSTER_SIZE;
	
	int MinRenderPassIndex = NumRayGuidingTrialSamples;

#if USE_IRRADIANCE_CACHING
	MinRenderPassIndex += GetIrradianceCachingQuality();
#endif
	
	if (BatchedTiles[TileIndex].RenderPassIndex >= MinRenderPassIndex)
	{
		float LastPrefixSum = 0;
		float PrefixSum;
		float LastRowPrefixSum = 0;
		float RowPrefixSum;
		
		int Y = floor(PrimarySample.y * DIRECTIONAL_BINS_ONE_DIM);	
		
		{
			int2 WritePos = int2(Y % 4, Y / 4);
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM / 4 + WritePos;
			
			RowPrefixSum = RayGuidingCDFY[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE / 4 + FinalPosition];
		}
		
		if (Y > 0)
		{
			int2 WritePos = int2((Y - 1) % 4, (Y - 1) / 4);
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM / 4 + WritePos;
			
			LastRowPrefixSum = RayGuidingCDFY[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE / 4 + FinalPosition];
		}
		
		PdfAdjust *= (RowPrefixSum - LastRowPrefixSum) * DIRECTIONAL_BINS_ONE_DIM;	
		
		int X = floor(PrimarySample.x * DIRECTIONAL_BINS_ONE_DIM);
		
		{
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + int2(X, Y);
			PrefixSum = RayGuidingCDFX[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition];
		}
		
		if (X > 0)
		{
			int2 FinalPosition = ClusterPosition * DIRECTIONAL_BINS_ONE_DIM + int2(X - 1, Y);
			LastPrefixSum = RayGuidingCDFX[BatchedTiles[TileIndex].WorkingSetPosition / GPreviewLightmapPhysicalTileSize * CDF_TILE_SIZE + FinalPosition];
		}
		
		PdfAdjust *= (PrefixSum - LastPrefixSum) * DIRECTIONAL_BINS_ONE_DIM;
	}
#endif	
#endif
	float NoL = max(dot(N_World, L_World), 0);

#ifdef LIGHTMAP_PATH_TRACING_MAIN_RG
	float OutPdf = NoL / PI * PdfAdjust;
#else
	float OutPdf = 1 / (2 * PI) * PdfAdjust;
#endif
	// As the name suggests, radiance probe has a combined throughput (Pdf * Weight) == 1
	float OutWeight = (OutPdf > 0.0f) ? (1.0f / OutPdf) : 0;

	return CreateMaterialEval(OutWeight, OutPdf);
}

#else

FMaterialSample RadianceProbe_SampleMaterial(
	FPathTracingPayload Payload,
	float3 RandSample)
{
	float3 N_World = Payload.WorldNormal;

	float4 SampledValue = CosineSampleHemisphere(RandSample.xy);

	return CreateMaterialSample(TangentToWorld(SampledValue.xyz, N_World), 1.0, SampledValue.w, 1.0, 1.0, PATHTRACER_SCATTER_DIFFUSE);
}

FMaterialEval RadianceProbe_EvalMaterial(
	float3 L_World,
	FPathTracingPayload Payload
)
{
	float3 N_World = Payload.WorldNormal;
	float NoL = saturate(dot(N_World, L_World));

	return CreateMaterialEval(Payload.DiffuseColor, NoL / PI);
}

#endif