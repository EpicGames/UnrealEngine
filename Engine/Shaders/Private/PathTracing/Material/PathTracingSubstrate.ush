// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingEnergyConservation.ush"
#include "PathTracingSubstrateCommon.ush"
#include "PathTracingMaterialCommon.ush"
#include "PathTracingGlossy.ush"
#include "PathTracingFresnel.ush"
#include "../../BRDF.ush"

struct FSubstrateSingleSlabData
{
	float3x3 Basis;
	float3 V;

	float  DiffuseWeight;
	float3 F0;
	float3 F90;
	float2 Alpha0;
	float2 Alpha1;
	float3 Spec0W;
	float3 Spec1W;
	FSubstrateSheenData Fuzz;

	float3 LobeCdf;
	float4 LobePdf;

	void PrepareBSDF(float3 V_World, FPathTracingPayload Payload, bool bUseSSSColor = false)
	{
		float3 NSpec = AdjustShadingNormal(Payload.WorldNormal, Payload.WorldGeoNormal, V_World);
		Basis = GetGGXBasis(Payload.RoughnessData.x, Payload.Anisotropy, NSpec, Payload.WorldTangent, Alpha0);
		Alpha1 = GetGGXAlpha(Payload.RoughnessData.y, Payload.Anisotropy);

		V = mul(Basis, V_World);

		// make sure F0=0.0 fades off the specular lobe completely
		F0 = Payload.SpecularColor;
		F90 = Payload.SpecularEdgeColor * F0RGBToMicroOcclusion(F0);

		const float NoV = saturate(V.z);
		// SUBSTRATE_TODO: Add support for F82 model?
		FBxDFEnergyTermsRGB Spec0 = ComputeGGXSpecEnergyTermsRGB(Payload.RoughnessData.x, NoV, F0, F90);
		FBxDFEnergyTermsRGB Spec1 = ComputeGGXSpecEnergyTermsRGB(Payload.RoughnessData.y, NoV, F0, F90);
		Fuzz.Prepare(V, Payload.FuzzRoughness, Payload.FuzzAmount);

		// save the parts needed during eval/sampling
		Spec0W = Spec0.W;
		Spec1W = Spec1.W;

		const float3 SpecE = lerp(Spec0.E, Spec1.E, Payload.RoughnessData.z);

		DiffuseWeight = Fuzz.Attenuation * (1.0 - Luminance(SpecE));

		const float3 Spec0Albedo = Fuzz.Attenuation * (1.0 - Payload.RoughnessData.z) * Spec0.E;
		const float3 Spec1Albedo = Fuzz.Attenuation * (      Payload.RoughnessData.z) * Spec1.E;
		const float3 FuzzAlbedo = Fuzz.Scale * Payload.FuzzColor;

		// Now prepare a cdf/pdf for lobe selection
		float3 MaxLobeWeight = Payload.GetMaxLobeWeight();
		LobeCdf = LobeSelectionCdf(
			MaxLobeWeight * DiffuseWeight * (bUseSSSColor ? Payload.SubsurfaceColor : Payload.DiffuseColor),
			MaxLobeWeight * Spec0Albedo,
			MaxLobeWeight * Spec1Albedo,
			MaxLobeWeight * FuzzAlbedo);
		LobePdf = LobeSelectionPdf(LobeCdf);
	}
};

FMaterialSample Substrate_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	FSubstrateSingleSlabData Data = (FSubstrateSingleSlabData)0;
	Data.PrepareBSDF(V_World, Payload);


	float3 L = 0, H = 0, V = Data.V;

	const bool bSampledDiffuse  = RandSample.x < Data.LobeCdf.x;
	const bool bSampledSpecular = RandSample.x < Data.LobeCdf.z;

	FMaterialSample Result = NullMaterialSample();
	Result.Roughness = 1;
	Result.PositionBiasSign = 1;
	if (bSampledDiffuse)
	{
		// diffuse lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, Data.LobeCdf.x);
		// cosine sampling without needing to form an orthonormal basis
		Result.Direction = CosineSampleHemisphereWorld(RandSample.xy, Payload.WorldNormal).xyz;

		// setup vectors for MIS math
		L = mul(Data.Basis, Result.Direction);
		H = normalize(L + V);
	}
	else if (bSampledSpecular)
	{
		// specular lobes
		const bool bUseSpec0 = RandSample.x < Data.LobeCdf.y;

		Result.Roughness = bUseSpec0 ? Payload.RoughnessData.x : Payload.RoughnessData.y;
		RandSample.x = RescaleRandomNumber(RandSample.x, bUseSpec0 ? Data.LobeCdf.x : Data.LobeCdf.y, bUseSpec0 ? Data.LobeCdf.y : Data.LobeCdf.z);
		H = ImportanceSampleVisibleGGX(RandSample.xy, bUseSpec0 ? Data.Alpha0 : Data.Alpha1, V).xyz;
		L = reflect(-V, H);
		if (L.z <= 0)
		{
			// invalid output direction, exit early
			return NullMaterialSample();
		}
		Result.Direction = mul(L, Data.Basis);
	}
	else
	{
		// cloth lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.z, 1.0);
		L = Data.Fuzz.Sample(RandSample.xy);
		H = normalize(L + V);
		Result.Direction = mul(L, Data.Basis);
	}

	// transform to world space
	const float3 L_World = Result.Direction;
	const float NoV = saturate(V.z);
	const float NoL = saturate(L.z);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);

	const float ShadowTerminator = ShadowTerminatorTerm(L_World, Payload.WorldNormal, Payload.WorldSmoothNormal);
	const float DiffNoL = saturate(dot(Result.Direction, Payload.WorldNormal));
	const float DiffPdf = DiffNoL / PI;
	float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0);
	const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1);
	const float Spec0Pdf = GGXResult0.y;
	const float Spec1Pdf = GGXResult1.y;
	const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
	const float ClothPdf = V.z > 0 ? ClothResult.w : 0;

	if (bSampledDiffuse)
	{
		float DiffNoL = saturate(dot(Payload.WorldNormal, L_World));
		float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		float3 H_World = normalize(V_World + L_World);
		float DiffVoH = saturate(dot(V_World, H_World));
		float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));

		const float3 Diffuse = Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);
		Result.AddLobeWithMIS(Diffuse * ShadowTerminator, DiffPdf, Data.LobePdf.x);

		Result.Pdf += Data.LobePdf.y * Spec0Pdf;
		Result.Pdf += Data.LobePdf.z * Spec1Pdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else if (bSampledSpecular)
	{
		SubstrateGlint(Payload, V, L, Data.Alpha0, GGXResult0.y /*Pdf*/, GGXResult0.x /*Weight*/);
		
		const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);
#if SUBSTRATE_FRESNEL_F82
		// SUBSTRATE_TODO: Need to tune the logic for specular micro-occlusion ...
		const float3 F = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, VoH) * FTint;
#else
		const float3 F = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, VoH) * FTint;
#endif

		Result.AddLobeWithMIS((1.0 - Payload.RoughnessData.z) * F * GGXResult0.x * Data.Spec0W, Spec0Pdf, Data.LobePdf.y);
		Result.AddLobeWithMIS((      Payload.RoughnessData.z) * F * GGXResult1.x * Data.Spec1W, Spec1Pdf, Data.LobePdf.z);
		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else
	{
		// Cloth Lobe
		Result.AddLobeWithMIS(ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.w);
		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.y * Spec0Pdf;
		Result.Pdf += Data.LobePdf.z * Spec1Pdf;
	}

	Result.ScatterType = bSampledDiffuse ? PATHTRACER_SCATTER_DIFFUSE : PATHTRACER_SCATTER_SPECULAR;

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);

	return Result;
}

FMaterialEval Substrate_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	FSubstrateSingleSlabData Data = (FSubstrateSingleSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	// move vectors into right shading frame
	const float3 V = Data.V;
	const float3 L = mul(Data.Basis, L_World);
	const float3 H = normalize(V + L);
	const float NoV = saturate(V.z);
	const float NoL = saturate(L.z);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);

	FMaterialEval Result = NullMaterialEval();

	const float ShadowTerminator = ShadowTerminatorTerm(L_World, Payload.WorldNormal, Payload.WorldSmoothNormal);
	const float DiffNoL = saturate(dot(Payload.WorldNormal, L_World));
	const float DiffPdf = DiffNoL / PI;
	{
		const float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		const float3 H_World = normalize(V_World + L_World);
		const float DiffVoH = saturate(dot(V_World, H_World));
		const float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));
		const float3 Diffuse = Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);
		Result.AddLobeWithMIS(DiffuseSpecularScale.x * Diffuse * ShadowTerminator, DiffPdf, Data.LobePdf.x);
	}
	{
		float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0);
		const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1);
		const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);
#if SUBSTRATE_FRESNEL_F82
		const float3 F = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, VoH) * FTint;
#else
		const float3 F = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, VoH) * FTint;
#endif
		SubstrateGlint(Payload, V, L, Data.Alpha0, GGXResult0.y /*Pdf*/, GGXResult0.x /*Weight*/);

		Result.AddLobeWithMIS(DiffuseSpecularScale.y * (1.0 - Payload.RoughnessData.z) * F * GGXResult0.x * Data.Spec0W, GGXResult0.y, Data.LobePdf.y);
		Result.AddLobeWithMIS(DiffuseSpecularScale.y * (      Payload.RoughnessData.z) * F * GGXResult1.x * Data.Spec1W, GGXResult1.y, Data.LobePdf.z);
	}
	{
		// Cloth Lobe
		const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
		const float ClothPdf = ClothResult.w;
		Result.AddLobeWithMIS(DiffuseSpecularScale.y * ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.w);
	}

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);

	return Result;
}

struct FSubstrateClearCoatData
{
	float3x3 BaseBasis;
	float3x3 CoatBasis;

	float3 BaseV;
	float3 CoatV;

	float  DiffuseWeight;
	float3 F0;
	float3 F90;
	float2 BaseAlpha;
	float2 CoatAlpha;

	float CoatAttenuation;
	FBxDFEnergyTermsRGB BaseSpec;
	FBxDFEnergyTermsA   CoatSpec;
	FSubstrateSheenData Fuzz;

	float3 LobeCdf;
	float4 LobePdf;

	void PrepareBSDF(float3 V_World, FPathTracingPayload Payload, bool bUseSSSColor = false)
	{
		float3 CoatWorldNormal = Payload.WorldNormal;
		CoatWorldNormal = AdjustShadingNormal(CoatWorldNormal, Payload.WorldGeoNormal, V_World);
#if CLEAR_COAT_BOTTOM_NORMAL
		float3 BaseWorldNormal = Payload.GetBottomCoatNormal();
		BaseWorldNormal = AdjustShadingNormal(BaseWorldNormal, Payload.WorldGeoNormal, V_World);
#else
		float3 BaseWorldNormal = CoatWorldNormal;
#endif
		

		BaseBasis = GetGGXBasis(Payload.RoughnessData.x, Payload.Anisotropy, BaseWorldNormal, Payload.WorldTangent, BaseAlpha);
		BaseAlpha = GetGGXAlpha(Payload.RoughnessData.x, Payload.Anisotropy);

		CoatBasis = GetGGXBasis(Payload.RoughnessData.y, Payload.Anisotropy, CoatWorldNormal, Payload.WorldTangent, CoatAlpha);
		CoatAlpha = GetGGXAlpha(Payload.RoughnessData.y, Payload.Anisotropy);

		BaseV = mul(BaseBasis, V_World);
		CoatV = mul(CoatBasis, V_World);

		// make sure F0=0.0 fades off the specular lobe completely
		F0 = Payload.SpecularColor;
		F90 = Payload.SpecularEdgeColor * F0RGBToMicroOcclusion(F0);

		const float BaseNoV = saturate(BaseV.z);
		const float CoatNoV = saturate(CoatV.z);
		// SUBSTRATE_TODO: Add support for F82 model?
		BaseSpec = ComputeGGXSpecEnergyTermsRGB(Payload.RoughnessData.x, BaseNoV, F0, F90);
		CoatSpec = ComputeGGXSpecEnergyTermsA(Payload.RoughnessData.y, CoatNoV, CLEAR_COAT_F0);
		Fuzz.Prepare(CoatNoV, Payload.FuzzRoughness, Payload.FuzzAmount);

		CoatAttenuation = 1.0 - Payload.RoughnessData.z * CoatSpec.E;
		DiffuseWeight = Fuzz.Attenuation * CoatAttenuation * (1.0 - Luminance(BaseSpec.E));

		const float3 BaseSpecAlbedo = Fuzz.Attenuation * CoatAttenuation * BaseSpec.E;
		const float  CoatSpecAlbedo = Fuzz.Attenuation * Payload.RoughnessData.z * CoatSpec.E;
		const float3 FuzzAlbedo = Fuzz.Scale * Payload.FuzzColor;

		// Now prepare a cdf/pdf for lobe selection
		float3 MaxLobeWeight = Payload.GetMaxLobeWeight();
		LobeCdf = LobeSelectionCdf(
			MaxLobeWeight * DiffuseWeight * (bUseSSSColor ? Payload.SubsurfaceColor : Payload.DiffuseColor),
			MaxLobeWeight * BaseSpecAlbedo,
			MaxLobeWeight * CoatSpecAlbedo,
			MaxLobeWeight * FuzzAlbedo);
		LobePdf = LobeSelectionPdf(LobeCdf);
	}
};



FMaterialSample SubstrateClearCoat_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	FSubstrateClearCoatData Data = (FSubstrateClearCoatData)0;
	Data.PrepareBSDF(V_World, Payload);

	FMaterialSample Result = NullMaterialSample();
	Result.PositionBiasSign = 1;
	Result.Roughness = 1;
	const bool bSampledDiffuse = RandSample.x < Data.LobeCdf.x;
	const bool bSampledSpecular = RandSample.x < Data.LobeCdf.z;
	if (bSampledDiffuse)
	{
		RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, Data.LobeCdf.x);
		// diffuse lobe
		Result.Direction = CosineSampleHemisphereWorld(RandSample.xy, Payload.WorldNormal).xyz;
	}
	else if (bSampledSpecular)
	{
		// specular lobes
		const bool bUseBaseSpec = RandSample.x < Data.LobeCdf.y;
		if (bUseBaseSpec)
		{
			RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.x, Data.LobeCdf.y);
			Result.Roughness = Payload.RoughnessData.x;
		}
		else
		{
			RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.y, Data.LobeCdf.z);
			Result.Roughness = Payload.RoughnessData.y;
		}
		float3 V = bUseBaseSpec ? Data.BaseV : Data.CoatV;
		float3 H = ImportanceSampleVisibleGGX(RandSample.xy, bUseBaseSpec ? Data.BaseAlpha : Data.CoatAlpha, V).xyz;
		float3 L = reflect(-V, H);
		if (L.z <= 0)
		{
			// invalid output direction, exit early
			return NullMaterialSample();
		}
		Result.Direction = mul(L, bUseBaseSpec ? Data.BaseBasis : Data.CoatBasis);
	}
	else
	{
		// cloth lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.z, 1.0);
		float3 L = Data.Fuzz.Sample(RandSample.xy);
		Result.Direction = mul(L, Data.CoatBasis);
	}

	const float3 L_World = Result.Direction;
	const float3 BaseL = mul(Data.BaseBasis, L_World);
	const float3 CoatL = mul(Data.CoatBasis, L_World);
	const float3 BaseH = normalize(BaseL + Data.BaseV);
	const float3 CoatH = normalize(CoatL + Data.CoatV);

	const float BaseNoV = saturate(Data.BaseV.z);
	const float BaseNoL = saturate(BaseL.z);
	const float BaseVoH = saturate(dot(Data.BaseV, BaseH));
	const float BaseNoH = saturate(BaseH.z);

	const float CoatNoV = saturate(Data.CoatV.z);
	const float CoatNoL = saturate(CoatL.z);
	const float CoatVoH = saturate(dot(Data.CoatV, CoatH));
	const float CoatNoH = saturate(CoatH.z);

	const float Metallic = F0RGBToMetallic(Payload.SpecularColor);
	const float3 BaseColor = lerp(Payload.DiffuseColor, Payload.SpecularColor, Metallic);
	const float3 Transmission = lerp(1.0, SimpleClearCoatTransmittance(CoatNoL, CoatNoV, Metallic, BaseColor), Payload.RoughnessData.z);

	const float DiffNoL = saturate(dot(Payload.WorldNormal, L_World));
	const float DiffPdf = DiffNoL / PI;
	const float2 BaseGGXResult = GGXEvalReflection(BaseL, Data.BaseV, BaseH, Data.BaseAlpha);
	const float2 CoatGGXResult = GGXEvalReflection(CoatL, Data.CoatV, CoatH, Data.CoatAlpha);
	const float BaseSpecPdf = BaseGGXResult.y;
	const float CoatSpecPdf = CoatGGXResult.y;
	const float4 ClothResult = Data.Fuzz.Eval(CoatL, Data.CoatV, CoatH, Payload.FuzzColor);
	const float ClothPdf = ClothResult.w;

	const float ShadowTerminator = ShadowTerminatorTerm(L_World, Payload.WorldNormal, Payload.WorldSmoothNormal);
	if (bSampledDiffuse)
	{
		const float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		const float3 H_World = normalize(V_World + L_World);
		const float DiffVoH = saturate(dot(V_World, H_World));
		const float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));
		const float3 Diffuse = Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);

		Result.AddLobeWithMIS(Diffuse * ShadowTerminator, DiffPdf, Data.LobePdf.x);

		Result.Pdf += Data.LobePdf.y * BaseSpecPdf;
		Result.Pdf += Data.LobePdf.z * CoatSpecPdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else if (bSampledSpecular)
	{
		const float3 FTint = SubstrateSpecularTint(Payload, BaseNoV, BaseNoL, BaseVoH, BaseNoH);
#if SUBSTRATE_FRESNEL_F82
		// SUBSTRATE_TODO: Need to tune the logic for specular micro-occlusion ...
		const float3 FBase = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, BaseVoH) * FTint;
#else
		const float3 FBase = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, BaseVoH) * FTint;
#endif
		const float3 FCoat = Payload.RoughnessData.z * ClearCoatFresnel(CoatVoH) * FTint;

		Result.AddLobeWithMIS(Data.CoatAttenuation  * FBase * BaseGGXResult.x * Data.BaseSpec.W * Transmission, BaseSpecPdf, Data.LobePdf.y);
		Result.AddLobeWithMIS(Data.Fuzz.Attenuation * FCoat * CoatGGXResult.x * Data.CoatSpec.W               , CoatSpecPdf, Data.LobePdf.z);
		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else
	{
		// Cloth Lobe
		Result.AddLobeWithMIS(ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.w);

		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.y * BaseSpecPdf;
		Result.Pdf += Data.LobePdf.z * CoatSpecPdf;
	}

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);
	Result.ScatterType = bSampledDiffuse ? PATHTRACER_SCATTER_DIFFUSE : PATHTRACER_SCATTER_SPECULAR;

	return Result;
}


FMaterialEval SubstrateClearCoat_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	FSubstrateClearCoatData Data = (FSubstrateClearCoatData)0;
	Data.PrepareBSDF(V_World, Payload);

	// move vectors into right shading frame
	const float3 BaseV   = Data.BaseV;
	const float3 BaseL   = mul(Data.BaseBasis, L_World);
	const float3 BaseH   = normalize(BaseV + BaseL);
	const float  BaseNoV = saturate(BaseV.z);
	const float  BaseNoL = saturate(BaseL.z);
	const float  BaseVoH = saturate(dot(BaseV, BaseH));
	const float  BaseNoH = saturate(BaseH.z);

	const float3 CoatV   = Data.CoatV;
	const float3 CoatL   = mul(Data.CoatBasis, L_World);
	const float3 CoatH   = normalize(CoatV + CoatL);
	const float  CoatNoV = saturate(CoatV.z);
	const float  CoatNoL = saturate(CoatL.z);
	const float  CoatVoH = saturate(dot(CoatV, CoatH));
	const float  CoatNoH = saturate(CoatH.z);

	FMaterialEval Result = NullMaterialEval();

	const float Metallic = F0RGBToMetallic(Payload.SpecularColor);
	const float3 BaseColor = lerp(Payload.DiffuseColor, Payload.SpecularColor, Metallic);
	const float3 Transmission = lerp(1.0, SimpleClearCoatTransmittance(CoatNoL, CoatNoV, Metallic, BaseColor), Payload.RoughnessData.z);

	const float BasehadowTerminator  = ShadowTerminatorTerm(L_World, Payload.WorldNormal, Payload.WorldSmoothNormal);
	const float CoatShadowTerminator = ShadowTerminatorTerm(L_World, Data.CoatBasis[2] , Payload.WorldSmoothNormal);

	const float DiffNoL = saturate(dot(Payload.WorldNormal, L_World));
	{
		const float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		const float3 H_World = normalize(V_World + L_World);
		const float DiffVoH = saturate(dot(V_World, H_World));
		const float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));

		const float3 Diffuse = Transmission * Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);
		Result.AddLobeWithMIS(DiffuseSpecularScale.x * Diffuse * BasehadowTerminator, DiffNoL / PI, Data.LobePdf.x);
	}
	{
		const float2 BaseGGXResult = GGXEvalReflection(BaseL, BaseV, BaseH, Data.BaseAlpha);
		const float2 CoatGGXResult = GGXEvalReflection(CoatL, CoatV, CoatH, Data.CoatAlpha);
		const float BaseSpecPdf = BaseGGXResult.y;
		const float CoatSpecPdf = CoatGGXResult.y;

		const float3 FTint = SubstrateSpecularTint(Payload, BaseNoV, BaseNoL, BaseVoH, BaseNoH);
#if SUBSTRATE_FRESNEL_F82
		// SUBSTRATE_TODO: Need to tune the logic for specular micro-occlusion ...
		const float3 FBase = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, BaseVoH) * FTint;
#else
		const float3 FBase = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, BaseVoH) * FTint;
#endif
		const float3 FCoat = Payload.RoughnessData.z * ClearCoatFresnel(CoatVoH) * FTint;

		Result.AddLobeWithMIS(Data.CoatAttenuation  * FBase * BaseGGXResult.x * Data.BaseSpec.W * Transmission, BaseSpecPdf, Data.LobePdf.y);
		Result.AddLobeWithMIS(Data.Fuzz.Attenuation * FCoat * CoatGGXResult.x * Data.CoatSpec.W               , CoatSpecPdf, Data.LobePdf.z);
	}
	{
		// Cloth Lobe
		const float4 ClothResult = Data.Fuzz.Eval(CoatL, CoatV, CoatH, Payload.FuzzColor);
		const float ClothPdf = ClothResult.w;
		Result.AddLobeWithMIS(DiffuseSpecularScale.y * ClothResult.xyz * CoatShadowTerminator, ClothPdf, Data.LobePdf.w);
	}

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);

	return Result;
}
