// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../PathTracingCommon.ush"
#include "../../BRDF.ush"

#define SUBSTRATE_FRESNEL_F82  0

#if SUBSTRATE_PATH_TRACING_GLINTS
#include "/Engine/Private/Substrate/Glint/GlintThirdParty.ush"
#endif

float3 SubstrateLobeWeight(FPathTracingPayload Payload, float SatNoL)
{
	// See reference in LuminanceWeight
	const float DistanceL = rcp(max(SatNoL, 0.001f));
	const float3 TransmittanceAboveAlongL = select(Payload.TransmittanceN < 1.f, pow(max(Payload.TransmittanceN, 0.0001f), DistanceL), Payload.TransmittanceN);
	return Payload.BSDFOpacity * Payload.WeightV * lerp(1.0f, TransmittanceAboveAlongL, Payload.CoverageAboveAlongN);
}

float3 SubstrateSpecularTint(FPathTracingPayload Payload, float NoV, float NoL, float VoH, float NoH)
{
	float3 Out = 1.f;
#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
	BRANCH
	if (Payload.SpecularProfileId != 0)
	{
		Out = EvaluateSpecularProfile(Payload.SpecularProfileId, NoV, NoL, VoH, NoH);
	}
#endif
	return Out;
}

void SubstrateGlint(FPathTracingPayload Payload, float3 InTangentV, float3 InTangentL, float2 InAlphaXY, in float InPdf, inout float OutWeight)
{
	// For now use the GGX lobe sampling. This will be changed with glint importance sampling later on.
#if SUBSTRATE_PATH_TRACING_GLINTS
	BRANCH
	if (Payload.GlintValue < 1.f)
	{
		const float2 GGXRoughnessXY = float2(sqrtFast(InAlphaXY.x), sqrtFast(InAlphaXY.y));
		FBeckmannDesc Beckmann = GGXToBeckmann(GGXRoughnessXY);
		const float Fs = f_P(InTangentV, InTangentL, float3(Beckmann.Sigma, Beckmann.Rho), Payload.GlintValue, Payload.GlintUV, Payload.GlintUVdx, Payload.GlintUVdy, true /*bIncludeGeometryTerm*/);
		OutWeight = Fs / max(0.0001f, InPdf);
	}
#endif
}

#define USE_SHEEN_LTC_SAMPLER 1 // 0: fallback to uniform sampling (for reference)
								// 1: use the LTC coefficients for importance sampling

struct FSubstrateSheenData
{
	float2 Tangent; 	// Captures rotation within the local tangent frame
	float aInv; 		// LTC Coefficients
	float bInv;
	float Scale;      	// overall scale factor for the sheen lobe
	float Attenuation; 	// amount by which other lobes have to be scaled

	void Prepare(float3 V, float FuzzRoughness, float FuzzAmount)
	{
		Tangent = V.xy;
		const float Len2 = length2(Tangent);
		if (Len2 > 0)
		{
			Tangent *= rsqrt(Len2);
		}
		else
		{
			Tangent = float2(1, 0);
		}
		float4 LTC = SheenLTC_Cofficients(V.z, FuzzRoughness, View.SheenLTCTexture, View.SheenLTCSampler);
		aInv = LTC.x;
		bInv = LTC.y;
		const float DirectionalAlbedo = LTC.z;
		Scale = FuzzAmount * DirectionalAlbedo;
		Attenuation = 1 - FuzzAmount * DirectionalAlbedo; // equivalent to: lerp(1.0f, ComputeEnergyPreservation(EnergyTerms), FuzzAmount);
	}



	float4 Eval(float3 L, float3 V, float3 H, float3 FuzzColor)
	{
		if (L.z > 0)
		{
			const float3 LocalL = float3(
				L.x * Tangent.x + L.y * Tangent.y,
				L.y * Tangent.x - L.x * Tangent.y,
				L.z);

			const float3 WrappedLocalL = float3(
				aInv * LocalL.x + bInv * LocalL.z,
				aInv * LocalL.y,
				LocalL.z);

			const float InvLenWrappedLocalL2 = rcp(length2(WrappedLocalL));
			// Jabobian is defined as: det(InvLTC) / ||InvLTC . L||
			// Jacobian should be aInv^2/Length^3 but since we skip the normalization
			// we end up with aInv^2/Length^4 which is cheaper to compute
			const float Jacobian = Pow2(aInv * InvLenWrappedLocalL2);


			// Evaluation a normalized cos distribution (note: WrappedLocalL was not normalized, we folded the extra length division into the Jacobian)
#if USE_SHEEN_LTC_SAMPLER == 1
			const float Weight = Scale;
			const float Pdf = WrappedLocalL.z * Jacobian / PI;
#else
			const float Weight = Scale * WrappedLocalL.z * Jacobian * 2; // multiply by 2PI
			const float Pdf = 1.0 / (2 * PI);
#endif
			return float4(Weight * FuzzColor, Pdf);
		}
		return 0;
	}

	float3 Sample(float2 Rand)
	{
#if USE_SHEEN_LTC_SAMPLER == 1
		const float3 CosSample = CosineSampleHemisphere(Rand).xyz;
		// NOTE: The vector is scaled by aInv compared to the reference implementation to avoid unecessary divides
		const float3 LocalL = normalize(float3(CosSample.x - CosSample.z * bInv, CosSample.y, aInv * CosSample.z));
		return float3(
			LocalL.x * Tangent.x - LocalL.y * Tangent.y,
			LocalL.x * Tangent.y + LocalL.y * Tangent.x,
			LocalL.z);
#else
		return UniformSampleHemisphere(Rand).xyz;
#endif
	}
};

FBxDFEnergyTermsA ComputeSheenEnergyTerms(float NoV, float SheenRoughness)
{
	FBxDFEnergyTermsA Result;
	Result.E = SheenLTC_DirectionalAlbedo(NoV, SheenRoughness, View.SheenLTCTexture, View.SheenLTCSampler);
	Result.W = Result.E; // overall weight is the directional albedo
	return Result;
}
