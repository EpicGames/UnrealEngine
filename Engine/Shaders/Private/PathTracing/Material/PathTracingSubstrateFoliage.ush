// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingEnergyConservation.ush"
#include "PathTracingSubstrateCommon.ush"
#include "PathTracingMaterialCommon.ush"
#include "PathTracingGlossy.ush"
#include "PathTracingFresnel.ush"
#include "../../BRDF.ush"

struct FSubstrateFoliageSlabData
{
	float3x3 Basis;
	float3 V;

	float  DiffuseWeight;
	float  TransmissionProb; // for sampling the transmission lobe
	float3 F0;
	float3 F90;
	float2 Alpha0;
	float2 Alpha1;
	float3 Spec0W;
	float3 Spec1W;
	FSubstrateSheenData Fuzz;

	float3 LobeCdf;
	float4 LobePdf;

	void PrepareBSDF(float3 V_World, FPathTracingPayload Payload)
	{
		float3 NSpec = AdjustShadingNormal(Payload.WorldNormal, Payload.WorldGeoNormal, V_World);
		Basis = GetGGXBasis(Payload.RoughnessData.x, Payload.Anisotropy, NSpec, Payload.WorldTangent, Alpha0);
		Alpha1 = GetGGXAlpha(Payload.RoughnessData.y, Payload.Anisotropy);

		V = mul(Basis, V_World);

		// make sure F0=0.0 fades off the specular lobe completely
		F0 = Payload.SpecularColor;
		F90 = Payload.SpecularEdgeColor * F0RGBToMicroOcclusion(F0);

		const float NoV = saturate(V.z);
		// SUBSTRATE_TODO: Add support for F82 model?
		FBxDFEnergyTermsRGB Spec0 = ComputeGGXSpecEnergyTermsRGB(Payload.RoughnessData.x, NoV, F0, F90);
		FBxDFEnergyTermsRGB Spec1 = ComputeGGXSpecEnergyTermsRGB(Payload.RoughnessData.y, NoV, F0, F90);
		Fuzz.Prepare(V, Payload.FuzzRoughness, Payload.FuzzAmount);

		// save the parts needed during eval/sampling
		Spec0W = Spec0.W;
		Spec1W = Spec1.W;
		

		const float3 SpecE = lerp(Spec0.E, Spec1.E, Payload.RoughnessData.z);

		DiffuseWeight = Fuzz.Attenuation * (1.0 - Luminance(SpecE));

		const float3 Spec0Albedo = Fuzz.Attenuation * (1.0 - Payload.RoughnessData.z) * Spec0.E;
		const float3 Spec1Albedo = Fuzz.Attenuation * (      Payload.RoughnessData.z) * Spec1.E;
		const float3 FuzzAlbedo = Fuzz.Scale * Payload.FuzzColor;

		// Now prepare a cdf/pdf for lobe selection
		float3 MaxLobeWeight = Payload.GetMaxLobeWeight();
		LobeCdf = LobeSelectionCdf(
			MaxLobeWeight * DiffuseWeight * (Payload.GetFoliageTransmissionColor() + Payload.DiffuseColor),
			MaxLobeWeight * Spec0Albedo,
			MaxLobeWeight * Spec1Albedo,
			MaxLobeWeight * FuzzAlbedo);
		LobePdf = LobeSelectionPdf(LobeCdf);
		TransmissionProb = LobeSelectionProb(Payload.GetFoliageTransmissionColor(), Payload.DiffuseColor);
	}
};


FMaterialSample SubstrateFoliage_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	FSubstrateFoliageSlabData Data = (FSubstrateFoliageSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	FMaterialSample Result = NullMaterialSample();
	Result.Roughness = 1;
	Result.PositionBiasSign = 1;
	float3 V = Data.V, L = 0, H = 0;

	const float TransmitCdf = Data.LobeCdf.x * Data.TransmissionProb;
	const bool bSampledTransmit = RandSample.x < TransmitCdf;
	const bool bSampledDiffuse  = RandSample.x < Data.LobeCdf.x;
	const bool bSampledSpecular = RandSample.x < Data.LobeCdf.z;
	if (bSampledDiffuse)
	{
		RandSample.x = RescaleRandomNumber(RandSample.x, bSampledTransmit ? 0.0 : TransmitCdf, bSampledTransmit ? TransmitCdf : Data.LobeCdf.x);
		// diffuse lobe
		Result.Direction = CosineSampleHemisphereWorld(RandSample.xy, Payload.WorldNormal * (bSampledTransmit ? -1 : 1)).xyz;
		L = mul(Data.Basis, Result.Direction);
		H = normalize(L + V);
	}
	else if (bSampledSpecular)
	{
		// specular lobes
		const bool bUseSpec0 = RandSample.x < Data.LobeCdf.y;

		Result.Roughness = bUseSpec0 ? Payload.RoughnessData.x : Payload.RoughnessData.y;
		RandSample.x = RescaleRandomNumber(RandSample.x, bUseSpec0 ? Data.LobeCdf.x : Data.LobeCdf.y, bUseSpec0 ? Data.LobeCdf.y : Data.LobeCdf.z);

		H = ImportanceSampleVisibleGGX(RandSample.xy, bUseSpec0 ? Data.Alpha0 : Data.Alpha1, V).xyz;
		L = reflect(-V, H);
		if (L.z <= 0)
		{
			// invalid output direction, exit early
			return NullMaterialSample();
		}
		Result.Direction = mul(L, Data.Basis);
	}
	else
	{
		// cloth lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.z, 1.0);
		L = Data.Fuzz.Sample(RandSample.xy);
		H = normalize(L + V);
		Result.Direction = mul(L, Data.Basis);
	}

	// transform to world space
	const float3 L_World = Result.Direction;
	const float NoV = saturate(V.z);
	const float NoL = abs(L.z);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);

	const float DiffNoL = abs(dot(Payload.WorldNormal, L_World));
	const float DiffPdf = DiffNoL / PI;
	const float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0);
	const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1);
	const float Spec0Pdf = GGXResult0.y;
	const float Spec1Pdf = GGXResult1.y;
	const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
	const float ClothPdf = ClothResult.w;

	const float ShadowTerminator = ShadowTerminatorTerm(L_World, Payload.WorldNormal, Payload.WorldSmoothNormal);
	if (bSampledTransmit)
	{
		Result.AddLobeWithMIS(Data.DiffuseWeight * Payload.GetFoliageTransmissionColor(), DiffPdf, Data.LobePdf.x * Data.TransmissionProb);
		Result.PositionBiasSign = -1;
	}
	else if (bSampledDiffuse)
	{
		const float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		const float3 H_World = normalize(V_World + L_World);
		const float DiffVoH = saturate(dot(V_World, H_World));
		const float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));
		const float3 Diffuse = Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);

		Result.AddLobeWithMIS(Diffuse * ShadowTerminator, DiffPdf, Data.LobePdf.x - Data.TransmissionProb * Data.LobePdf.x);

		Result.Pdf += Data.LobePdf.y * Spec0Pdf;
		Result.Pdf += Data.LobePdf.z * Spec1Pdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else if (bSampledSpecular)
	{
		const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);
#if SUBSTRATE_FRESNEL_F82
		// SUBSTRATE_TODO: Need to tune the logic for specular micro-occlusion ...
		const float3 F = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, VoH) * FTint;
#else
		const float3 F = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, VoH) * FTint;
#endif

		Result.AddLobeWithMIS((1.0 - Payload.RoughnessData.z) * F * GGXResult0.x * Data.Spec0W, Spec0Pdf, Data.LobePdf.y);
		Result.AddLobeWithMIS((      Payload.RoughnessData.z) * F * GGXResult1.x * Data.Spec1W, Spec1Pdf, Data.LobePdf.z);
		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.w * ClothPdf;
	}
	else
	{
		// Cloth Lobe
		Result.AddLobeWithMIS(ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.w);
		Result.Pdf += Data.LobePdf.x * DiffPdf;
		Result.Pdf += Data.LobePdf.y * Spec0Pdf;
		Result.Pdf += Data.LobePdf.z * Spec1Pdf;
	}

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);
	Result.ScatterType = bSampledDiffuse ? PATHTRACER_SCATTER_DIFFUSE : PATHTRACER_SCATTER_SPECULAR;
	return Result;
}

FMaterialEval SubstrateFoliage_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	FSubstrateFoliageSlabData Data = (FSubstrateFoliageSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	const float3 N_World = Payload.WorldNormal;

	// move vectors into right shading frame
	const float3 V = Data.V;
	const float3 L = mul(Data.Basis, L_World);
	const float3 H = normalize(V + L);
	const float NoV = saturate(V.z);
	const float NoL = abs(L.z);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);

	FMaterialEval Result = NullMaterialEval();

	const float DiffNoL = dot(Payload.WorldNormal, L_World);
	const float DiffPdf = DiffNoL / PI;
	const float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
	if (DiffNoL < 0)
	{
		const float3 Diffuse = Data.DiffuseWeight * Payload.GetFoliageTransmissionColor();
		Result.AddLobeWithMIS(DiffuseSpecularScale.x * Diffuse, -DiffPdf, Data.LobePdf.x * Data.TransmissionProb);
	}
	else
	{
		const float DiffNoV = saturate(dot(Payload.WorldNormal, V_World));
		const float3 H_World = normalize(V_World + L_World);
		const float DiffVoH = saturate(dot(V_World, H_World));
		const float DiffNoH = saturate(dot(Payload.WorldNormal, H_World));
		const float3 Diffuse = Data.DiffuseWeight * GetPathTracingDiffuseModel(Payload.DiffuseColor, Payload.RoughnessData.x, DiffNoV, DiffNoL, DiffVoH, DiffNoH);
		Result.AddLobeWithMIS(DiffuseSpecularScale.x * Diffuse * ShadowTerminator, DiffPdf, Data.LobePdf.x - Data.LobePdf.x * Data.TransmissionProb);
	}
	if (L.z > 0)
	{
		{
			float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0);
			const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1);
			const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);
			#if SUBSTRATE_FRESNEL_F82
			const float3 F = Data.Fuzz.Attenuation * F_AdobeF82(Data.F0, Data.F90, VoH) * FTint;
			#else
			const float3 F = Data.Fuzz.Attenuation * F_Schlick(Data.F0, Data.F90, VoH) * FTint;
			#endif

			Result.AddLobeWithMIS(DiffuseSpecularScale.y * (1.0 - Payload.RoughnessData.z) * F * GGXResult0.x * Data.Spec0W, GGXResult0.y, Data.LobePdf.y);
			Result.AddLobeWithMIS(DiffuseSpecularScale.y * (      Payload.RoughnessData.z) * F * GGXResult1.x * Data.Spec1W, GGXResult1.y, Data.LobePdf.z);
		}
		{
			// Cloth Lobe
			const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
			const float ClothPdf = ClothResult.w;
			Result.AddLobeWithMIS(DiffuseSpecularScale.y * ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.w);
		}
	}

	Result.Weight *= SubstrateLobeWeight(Payload, DiffNoL);

	return Result;
}