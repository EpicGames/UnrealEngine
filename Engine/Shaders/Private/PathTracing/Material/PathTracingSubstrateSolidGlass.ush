// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingEnergyConservation.ush"
#include "PathTracingSubstrateCommon.ush"
#include "PathTracingMaterialCommon.ush"
#include "PathTracingGlossy.ush"
#include "PathTracingFresnel.ush"
#include "../../BRDF.ush"


struct FSubstrateSolidGlassSlabData
{
	float3x3 Basis;
	float3 V;

	float  F0;
	float  Eta;
	float2 Alpha0;
	float2 Alpha1;

	float  EGlass0;
	float  EGlass1;

	FSubstrateSheenData Fuzz;

	float2 LobeCdf;
	float3 LobePdf;

	void PrepareBSDF(float3 V_World, FPathTracingPayload Payload)
	{
		float3 N = AdjustShadingNormal(Payload.WorldNormal, Payload.WorldGeoNormal, V_World);
		Basis = GetGGXBasis(Payload.RoughnessData.x, Payload.Anisotropy, N, Payload.WorldTangent, Alpha0);
		Alpha1 = GetGGXAlpha(Payload.RoughnessData.y, Payload.Anisotropy);

		V = mul(Basis, V_World);

		// #dxr_todo: Maintain a refraction stack on the path tracing payload
		F0 = F0RGBToF0(Payload.SpecularColor);
		// NOTE: IsFrontFace() determines polygon orientation, because the normal is always flipped towards in the incoming ray
		Eta = Payload.IsFrontFace() ? Payload.Ior : rcp(Payload.Ior);
		
		const float NoV = saturate(V.z);
		// correct for energy loss by scaling the whole BSDF
		EGlass0 = GGXEnergyLookup(Payload.RoughnessData.x, NoV, Eta);
		EGlass1 = GGXEnergyLookup(Payload.RoughnessData.y, NoV, Eta);

		Fuzz.Prepare(V, Payload.FuzzRoughness, Payload.FuzzAmount);

		const float3 Glass0Albedo = Fuzz.Attenuation * (1.0 - Payload.RoughnessData.z);
		const float3 Glass1Albedo = Fuzz.Attenuation * (      Payload.RoughnessData.z);
		const float3 FuzzAlbedo = Fuzz.Scale * Payload.FuzzColor;

		// Now prepare a cdf/pdf for lobe selection
		float3 MaxLobeWeight = Payload.GetMaxLobeWeight();
		LobeCdf = LobeSelectionCdf(
			MaxLobeWeight * Glass0Albedo,
			MaxLobeWeight * Glass1Albedo,
			MaxLobeWeight * FuzzAlbedo);
		LobePdf = LobeSelectionPdf(LobeCdf);
	}
};


FMaterialSample SubstrateSolidGlass_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	FSubstrateSolidGlassSlabData Data = (FSubstrateSolidGlassSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	float3 L = 0, H = 0, V = Data.V;
	float OutRoughness = 1;

	const bool bSampledSpecular = RandSample.x < Data.LobeCdf.y;
	if (bSampledSpecular)
	{
		// specular lobes
		const bool bUseSpec0 = RandSample.x < Data.LobeCdf.x;
		if (bUseSpec0)
		{
			RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, Data.LobeCdf.x);
			OutRoughness = Payload.RoughnessData.x;
		}
		else
		{
			RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.x, Data.LobeCdf.y);
			OutRoughness = Payload.RoughnessData.y;
		}

		H = ImportanceSampleVisibleGGX(RandSample.xy, bUseSpec0 ? Data.Alpha0 : Data.Alpha1, V, false).xyz;

		// Glass lobe (reflection and refraction)
		float3 L = 0;
		float F = 0;

		const bool bRefract = SampleRefraction(-V, H, Data.Eta, Data.F0, RandSample.z, L, F);
		// transform to world space
		const float3 L_World = normalize(mul(L, Data.Basis));

		FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, bRefract ? -1.0 : 1.0, OutRoughness, bRefract ? PATHTRACER_SCATTER_REFRACT : PATHTRACER_SCATTER_SPECULAR);

		const float2 GGXResult0 = bRefract ? GGXEvalRefraction(L, V, H, Data.Alpha0, Data.Eta) : GGXEvalReflection(L, V, H, Data.Alpha0, false);
		const float2 GGXResult1 = bRefract ? GGXEvalRefraction(L, V, H, Data.Alpha1, Data.Eta) : GGXEvalReflection(L, V, H, Data.Alpha1, false);

		Result.AddLobeWithMIS((1.0 - Payload.RoughnessData.z) * GGXResult0.x / Data.EGlass0, F * GGXResult0.y, Data.LobePdf.x);
		Result.AddLobeWithMIS((      Payload.RoughnessData.z) * GGXResult1.x / Data.EGlass1, F * GGXResult1.y, Data.LobePdf.y);

		if (bRefract)
		{
			// only need to account for overall BSDF scaling
			Result.Weight *= Data.Fuzz.Attenuation * Payload.BSDFOpacity * Payload.WeightV;
		}
		else
		{
			// Specular profile tinting
			const float NoL = saturate(L.z);
			const float NoV = saturate(V.z);
			const float VoH = saturate(dot(V, H));
			const float NoH = saturate(H.z);
			const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);

			// reflection side, apply full lobe weight and attenuation along L
			// and account for probability of picking the fuzz layer
			const float ClothPdf = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor).w;
			Result.Pdf += Data.LobePdf.z * ClothPdf;
			Result.Weight *= Data.Fuzz.Attenuation * SubstrateLobeWeight(Payload, NoL) * FTint;
		}

		return Result;
	}
	else
	{
		// cloth lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.y, 1.0);
		L = Data.Fuzz.Sample(RandSample.xy);
		H = normalize(L + V);

		// transform to world space
		const float3 L_World = normalize(mul(L, Data.Basis));
		const float3 N_World = Payload.WorldNormal;
		const float NoV = saturate(V.z);
		const float NoL = saturate(L.z);
		const float VoH = saturate(dot(V, H));
		const float NoH = saturate(H.z);


		const float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0);
		const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1);
		const float Fg = FresnelReflectance(VoH, Data.Eta, Data.F0);
		const float Spec0Pdf = Fg * GGXResult0.y;
		const float Spec1Pdf = Fg * GGXResult1.y;

		FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, 1.0, 1.0, PATHTRACER_SCATTER_SPECULAR);

		// Cloth Lobe
		const float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
		const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
		const float ClothPdf = ClothResult.w;
		Result.AddLobeWithMIS(ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.z);
		Result.ScatterType = PATHTRACER_SCATTER_SPECULAR;

		Result.Pdf += Data.LobePdf.y * Spec0Pdf;
		Result.Pdf += Data.LobePdf.z * Spec1Pdf;

		Result.Weight *= SubstrateLobeWeight(Payload, NoL);
		return Result;
	}
}


FMaterialEval SubstrateSolidGlass_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	FSubstrateSolidGlassSlabData Data = (FSubstrateSolidGlassSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	const float3 N_World = Payload.WorldNormal;

	// move vectors into right shading frame
	const float3 V = Data.V;
	const float3 L = mul(Data.Basis, L_World);

	FMaterialEval Result = NullMaterialEval();
	if (L.z > 0.0)
	{
		// Evaluate reflection lobes
		const float3 H = normalize(V + L);
		const float NoV = saturate(V.z);
		const float NoL = saturate(L.z);
		const float VoH = saturate(dot(V, H));
		const float NoH = saturate(H.z);


		{
			// Specular Lobes
			const float2 GGXResult0 = GGXEvalReflection(L, V, H, Data.Alpha0, false);
			const float2 GGXResult1 = GGXEvalReflection(L, V, H, Data.Alpha1, false);

			const float Fg = FresnelReflectance(VoH, Data.Eta, Data.F0);
			const float3 FTint = SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);

			Result.AddLobeWithMIS(FTint * DiffuseSpecularScale.y * Data.Fuzz.Attenuation * (1.0 - Payload.RoughnessData.z) * GGXResult0.x / Data.EGlass0, Fg * GGXResult0.y, Data.LobePdf.x);
			Result.AddLobeWithMIS(FTint * DiffuseSpecularScale.y * Data.Fuzz.Attenuation * (      Payload.RoughnessData.z) * GGXResult1.x / Data.EGlass1, Fg * GGXResult1.y, Data.LobePdf.y);
		}
		{
			// Cloth Lobe
			const float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
			const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
			const float ClothPdf = ClothResult.w;
			Result.AddLobeWithMIS(DiffuseSpecularScale.y * ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.z);
		}

		Result.Weight *= SubstrateLobeWeight(Payload, NoL);
	}
	else if (L.z < 0)
	{
		// Evaluate refracted lobes
		// refraction side
		const float NoL = saturate(-L.z);
		float3 Ht = -(Data.Eta * L + V);
		Ht = normalize((Data.Eta < 1.0f) ? -Ht : Ht);

		const float VoH = dot(V, Ht);
		const float Fg = 1.0f - FresnelReflectance(VoH, Data.Eta, Data.F0);
		if (Fg > 0)
		{
			const float2 GGXResult0 = GGXEvalRefraction(L, V, Ht, Data.Alpha0, Data.Eta);
			const float2 GGXResult1 = GGXEvalRefraction(L, V, Ht, Data.Alpha1, Data.Eta);

			Result.AddLobeWithMIS((1.0 - Payload.RoughnessData.z) * GGXResult0.x / Data.EGlass0 * DiffuseSpecularScale.y, GGXResult0.y * Fg, Data.LobePdf.x);
			Result.AddLobeWithMIS((      Payload.RoughnessData.z) * GGXResult1.x / Data.EGlass1 * DiffuseSpecularScale.y, GGXResult1.y * Fg, Data.LobePdf.y);
		}

		Result.Weight *= Payload.BSDFOpacity * Payload.WeightV * Data.Fuzz.Attenuation;
	}
	return Result;
}
