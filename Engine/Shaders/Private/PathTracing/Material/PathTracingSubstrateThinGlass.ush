// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingEnergyConservation.ush"
#include "PathTracingSubstrateCommon.ush"
#include "PathTracingMaterialCommon.ush"
#include "PathTracingGlossy.ush"
#include "PathTracingFresnel.ush"
#include "../../BRDF.ush"
#include "../../ParticipatingMediaCommon.ush"


struct FSubstrateThinGlassSlabData
{
	float3x3 Basis;
	float3 V;

	float2 RoughnessT;

	float  DiffuseWeight;
	float  F0, F90;
	float  Eta;
	float3 TransmissionColor;
	float2 AlphaR0, AlphaT0;
	float2 AlphaR1, AlphaT1;

	float SpecR0W, SpecR1W;
	float SpecT0W, SpecT1W;

	FSubstrateSheenData Fuzz;

	float3 LobeCdf;
	float4 LobePdf;

	void PrepareBSDF(float3 V_World, FPathTracingPayload Payload)
	{
		float2 RoughnessR = Payload.RoughnessData.xy;
		RoughnessT.x = ComputeThinTransmittedRoughness(RoughnessR.x, Payload.Ior);
		RoughnessT.y = ComputeThinTransmittedRoughness(RoughnessR.y, Payload.Ior);

		float3 N = AdjustShadingNormal(Payload.WorldNormal, Payload.WorldGeoNormal, V_World);

		Basis   = GetGGXBasis(RoughnessR.x, Payload.Anisotropy, N, Payload.WorldTangent, AlphaR0);
		AlphaR1 = GetGGXAlpha(RoughnessR.y, Payload.Anisotropy);

		AlphaT0 = GetGGXAlpha(RoughnessT.x, Payload.Anisotropy);
		AlphaT1 = GetGGXAlpha(RoughnessT.y, Payload.Anisotropy);

		V = mul(Basis, V_World);

		F0 = F0RGBToF0(Payload.SpecularColor);
		F90 = F0ToMicroOcclusion(F0);
		TransmissionColor = Payload.GetTransmittanceColor();

		const float NoV = saturate(V.z);
		FBxDFEnergyTermsA SpecR0 = ComputeGGXSpecEnergyTermsA(RoughnessR.x, NoV, F0, F90);
		FBxDFEnergyTermsA SpecR1 = ComputeGGXSpecEnergyTermsA(RoughnessR.y, NoV, F0, F90);
		FBxDFEnergyTermsA SpecT0 = ComputeGGXSpecEnergyTermsA(RoughnessT.x, NoV, 1.0 - F0, 1.0 - F90);
		FBxDFEnergyTermsA SpecT1 = ComputeGGXSpecEnergyTermsA(RoughnessT.y, NoV, 1.0 - F0, 1.0 - F90);
		float2 NudgeE = float2(
			rcp(SpecR0.E + SpecT0.E),
			rcp(SpecR1.E + SpecT1.E));

		Fuzz.Prepare(V, Payload.FuzzRoughness, Payload.FuzzAmount);



		const float3 GlassR = Fuzz.Attenuation * lerp(SpecR0.E * NudgeE.x, SpecR1.E * NudgeE.y, Payload.RoughnessData.z);
		const float3 GlassT = Fuzz.Attenuation * lerp(SpecT0.E * NudgeE.x, SpecT1.E * NudgeE.y, Payload.RoughnessData.z);
		const float3 FuzzAlbedo = Fuzz.Scale * Payload.FuzzColor;

		DiffuseWeight = Fuzz.Attenuation * (1.0 - Luminance(GlassR));

		SpecR0W = SpecR0.W * NudgeE.x;
		SpecR1W = SpecR1.W * NudgeE.y;
		SpecT0W = SpecT0.W * NudgeE.x;
		SpecT1W = SpecT1.W * NudgeE.y;

		// Now prepare a cdf/pdf for lobe selection
		// NOTE: we put T first so that the reflected lobe (R and fuzz) are consecutive in the PDF
		float3 MaxLobeWeight = Payload.GetMaxLobeWeight();
		LobeCdf = LobeSelectionCdf(
			MaxLobeWeight * DiffuseWeight * Payload.DiffuseColor,
			MaxLobeWeight * FuzzAlbedo,
			MaxLobeWeight * GlassR,
			Payload.Ior > 0.0 ? MaxLobeWeight * GlassT : 0.0);
		LobePdf = LobeSelectionPdf(LobeCdf);
	}
};

FMaterialSample SubstrateThinGlass_SampleMaterial(
	float3 V_World,
	FPathTracingPayload Payload,
	float3 RandSample
)
{
	FSubstrateThinGlassSlabData Data = (FSubstrateThinGlassSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	float3 L = 0, H = 0, V = Data.V, ScatterWeight = 0;
	float OutRoughness = 1;

	bool bSampleScatter = false, bSampleCloth = false, bSampleGlass = false, bIsReflection = RandSample.x < Data.LobeCdf.z;
	
	if (RandSample.x < Data.LobeCdf.x)
	{
		bSampleScatter = true;
		RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, Data.LobeCdf.x);
		// scattering lobe
		L = TangentToWorld(ImportanceSampleHenyeyGreensteinPhase(RandSample.xy, Payload.PhaseG).xyz, -V);

		ScatterWeight = HanrahanKrueger_Weight(V.z, L.z, Data.TransmissionColor);
		bIsReflection = L.z > 0;

		// keep L to positive side for the remainder of the calculations
		L.z = abs(L.z);
		H = normalize(L + V);
	}
	else if (RandSample.x < Data.LobeCdf.y)
	{
		bSampleCloth = true;
		// cloth lobe
		RandSample.x = RescaleRandomNumber(RandSample.x, Data.LobeCdf.x, Data.LobeCdf.y);
		L = Data.Fuzz.Sample(RandSample.xy);
		H = normalize(L + V);
	}
	else
	{
		bSampleGlass = true;
		// Two-phase lobe selection:
		// First use the overall CDF to pick between transmission and reflection ...
		RandSample.x = RescaleRandomNumber(RandSample.x,
										   bIsReflection ? Data.LobeCdf.y : Data.LobeCdf.z,
										   bIsReflection ? Data.LobeCdf.z : 1.0);

		// Then use the roughness blend to pick one of the two possible roughness values
		const bool bUseRoughness1 = RandSample.x < Payload.RoughnessData.z;
		RandSample.x = RescaleRandomNumber(RandSample.x,
										   bUseRoughness1 ? 0.0 : Payload.RoughnessData.z,
										   bUseRoughness1 ? Payload.RoughnessData.z : 1.0);

		// 
		float2 Alpha = bIsReflection ? (bUseRoughness1 ? Data.AlphaR1 : Data.AlphaR0) : (bUseRoughness1 ? Data.AlphaT1 : Data.AlphaT0);
		OutRoughness = bIsReflection ?
			(bUseRoughness1 ? Payload.RoughnessData.y : Payload.RoughnessData.x) :
			(bUseRoughness1 ? Data.RoughnessT.y : Data.RoughnessT.x);

		H = ImportanceSampleVisibleGGX(RandSample.xy, Alpha, V).xyz;
		L = reflect(-V, H);
		if (L.z <= 0)
		{
			// invalid output direction, exit early
			return NullMaterialSample();
		}

	}

	// transform to world space
	const float3 N_World = Payload.WorldNormal;
	const float NoV = saturate(V.z);
	const float NoL = saturate(L.z);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);

	const float PositionBiasSign = bIsReflection ? 1.0 : -1.0;
	const float3 L_World = normalize(mul(float3(L.xy, PositionBiasSign * L.z), Data.Basis)); // flip reflection to other side if we are refracting
	FMaterialSample Result = CreateMaterialSample(L_World, 0.0, 0.0, PositionBiasSign, OutRoughness, bSampleScatter ? PATHTRACER_SCATTER_DIFFUSE : (bIsReflection ? PATHTRACER_SCATTER_SPECULAR : PATHTRACER_SCATTER_REFRACT));

	// Specular Lobes (either reflected or transmitted)
	const float2 GGXResult0 = GGXEvalReflection(L, V, H, bIsReflection ? Data.AlphaR0 : Data.AlphaT0);
	const float2 GGXResult1 = GGXEvalReflection(L, V, H, bIsReflection ? Data.AlphaR1 : Data.AlphaT1);
	const float4 ClothResult = bIsReflection ? Data.Fuzz.Eval(L, V, H, Payload.FuzzColor) : 0;
	const float ClothPdf = ClothResult.w;
	const float ScatterPdf = HenyeyGreensteinPhase(Payload.PhaseG, dot(V_World, L_World));
	if (bSampleGlass)
	{
		FThinSlabWeights SlabResult = ComputeThinSlabWeights(Data.TransmissionColor, VoH, Payload.Ior, Data.F0);
		SlabResult.Reflected *= SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);

		const float3 SlabTint = bIsReflection ? SlabResult.Reflected : SlabResult.Transmitted;
		const float SlabW0    = bIsReflection ? Data.SpecR0W : Data.SpecT0W;
		const float SlabW1    = bIsReflection ? Data.SpecR1W : Data.SpecT1W;
		const float LobePdf   = bIsReflection ? Data.LobePdf.z : Data.LobePdf.w;

		Result.AddLobeWithMIS(Data.Fuzz.Attenuation * (1 - Payload.RoughnessData.z) * GGXResult0.x * SlabW0 * SlabTint, GGXResult0.y, LobePdf * (1 - Payload.RoughnessData.z));
		Result.AddLobeWithMIS(Data.Fuzz.Attenuation * (    Payload.RoughnessData.z) * GGXResult1.x * SlabW1 * SlabTint, GGXResult1.y, LobePdf * (    Payload.RoughnessData.z));

		if (bIsReflection)
		{
			Result.Pdf += Data.LobePdf.y * ClothPdf;
		}
		Result.Pdf += Data.LobePdf.x * ScatterPdf;
	}
	else if (bSampleScatter)
	{
		// Scattering Lobe
		Result.AddLobeWithMIS(Data.DiffuseWeight * Payload.DiffuseColor * ScatterWeight, ScatterPdf, Data.LobePdf.x);
		Result.Pdf += Data.LobePdf.y * ClothPdf;
		Result.Pdf += Data.LobePdf.z * (1.0 - Payload.RoughnessData.z) * GGXResult0.y;
		Result.Pdf += Data.LobePdf.z * (      Payload.RoughnessData.z) * GGXResult1.y;
	}
	else
	{
		// Cloth Lobe
		const float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
		Result.AddLobeWithMIS(ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.y);

		Result.Pdf += Data.LobePdf.x * ScatterPdf;
		Result.Pdf += Data.LobePdf.z * (1.0 - Payload.RoughnessData.z) * GGXResult0.y;
		Result.Pdf += Data.LobePdf.z * (      Payload.RoughnessData.z) * GGXResult1.y;
	}


	if (bIsReflection)
	{
		Result.Weight *= SubstrateLobeWeight(Payload, NoL);
	}
	else
	{
		Result.Weight *= Payload.BSDFOpacity * Payload.WeightV;
	}
	return Result;
}


FMaterialEval SubstrateThinGlass_EvalMaterial(
	float3 V_World,
	float3 L_World,
	FPathTracingPayload Payload,
	float2 DiffuseSpecularScale
)
{
	FSubstrateThinGlassSlabData Data = (FSubstrateThinGlassSlabData)0;
	Data.PrepareBSDF(V_World, Payload);

	const float3 N_World = Payload.WorldNormal;

	// move vectors into right shading frame
	const float3 V = Data.V;
	float3 L = mul(Data.Basis, L_World);

	FMaterialEval Result = NullMaterialEval();

	// Scattering Lobe
	const float ScatterPdf = HenyeyGreensteinPhase(Payload.PhaseG, dot(V_World, L_World));
	const float3 ScatterWeight = HanrahanKrueger_Weight(V.z, L.z, Data.TransmissionColor);
	Result.AddLobeWithMIS(DiffuseSpecularScale.x * Data.DiffuseWeight * Payload.DiffuseColor * ScatterWeight, ScatterPdf, Data.LobePdf.x);

	const bool bIsReflection = L.z >= 0;
	L.z = abs(L.z); // push L to the same side as V

	const float NoL = saturate(L.z);
	const float NoV = saturate(V.z);
	const float3 H = normalize(L + V);
	const float VoH = saturate(dot(V, H));
	const float NoH = saturate(H.z);


	// Specular Lobes (either reflected or transmitted)
	const float2 GGXResult0 = GGXEvalReflection(L, V, H, bIsReflection ? Data.AlphaR0 : Data.AlphaT0);
	const float2 GGXResult1 = GGXEvalReflection(L, V, H, bIsReflection ? Data.AlphaR1 : Data.AlphaT1);
	FThinSlabWeights SlabResult = ComputeThinSlabWeights(Data.TransmissionColor, VoH, Payload.Ior, Data.F0);
	SlabResult.Reflected *= SubstrateSpecularTint(Payload, NoV, NoL, VoH, NoH);

	const float3 SlabTint = bIsReflection ? SlabResult.Reflected : SlabResult.Transmitted;
	const float SlabW0 = (bIsReflection ? Data.SpecR0W : Data.SpecT0W);
	const float SlabW1 = (bIsReflection ? Data.SpecR1W : Data.SpecT1W);

	const float LobePdf = bIsReflection ? Data.LobePdf.z : Data.LobePdf.w;

	Result.AddLobeWithMIS(DiffuseSpecularScale.y * Data.Fuzz.Attenuation * (1 - Payload.RoughnessData.z) * GGXResult0.x * SlabW0 * SlabTint, GGXResult0.y, LobePdf * (1 - Payload.RoughnessData.z));
	Result.AddLobeWithMIS(DiffuseSpecularScale.y * Data.Fuzz.Attenuation * (    Payload.RoughnessData.z) * GGXResult1.x * SlabW1 * SlabTint, GGXResult1.y, LobePdf * (    Payload.RoughnessData.z));


	if (bIsReflection)
	{
		{
			// Cloth Lobe
			const float ShadowTerminator = ShadowTerminatorTerm(L_World, N_World, Payload.WorldSmoothNormal);
			const float4 ClothResult = Data.Fuzz.Eval(L, V, H, Payload.FuzzColor);
			const float ClothPdf = ClothResult.w;
			Result.AddLobeWithMIS(DiffuseSpecularScale.y * ClothResult.xyz * ShadowTerminator, ClothPdf, Data.LobePdf.y);
		}
		Result.Weight *= SubstrateLobeWeight(Payload, NoL);
	}
	else
	{
		Result.Weight *= Payload.BSDFOpacity * Payload.WeightV;
	}
	return Result;
}
