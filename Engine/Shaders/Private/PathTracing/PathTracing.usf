// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracing.usf: Reference path tracing
===============================================================================================*/

#define PATH_TRACING  1

#define PATH_TRACING_SHADER_VERSION 0x51478b68 // Bump to force-compile this shader

// Ensure specular profile and glints are supported
#if SUBSTRATE_ENABLED && PATH_TRACER_USE_SUBSTRATE_SPECIAL_COMPLEX_MATERIAL
#define SUBSTRATE_COMPLEXSPECIALPATH 1
#endif

#include "PathTracingCore.ush"

// describe how we split up the image between tiles and amongst GPUs
int2 TilePixelOffset;
int2 TileTextureOffset;
int ScanlineStride;
int ScanlineWidth;


// Which pixel on screen are we computing?
// We want to spread out the scanlines evenly across the screen so that each GPU works on a similarly sized task
int2 ComputePixelIndex(uint2 DispatchIdx)
{
	return DispatchIdx * int2(1, ScanlineStride) + TilePixelOffset;
}

// Where should we store the accumulated pixels of this dispatch?
// We want all the scanlines computed by a given GPU to be contiguous in memory so we can copy them easily
int2 ComputeTextureIndex(uint2 DispatchIdx)
{
	return DispatchIdx + TileTextureOffset;
}

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
int FirstBounce;
#endif

RWStructuredBuffer<FPathTracingPackedPathState> PathStateData;
RWBuffer<uint> ActivePaths;
RWBuffer<uint> NumPathStates;

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
RWBuffer<uint> NextActivePaths;
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
FPathState LoadPathStateData(uint Index)
{
	FPathTracingPackedPathState Data = PathStateData[Index];
	FPathState Output = (FPathState)0;
	Output.RandSequence.SampleIndex = Data.RandSeqSampleIndex;
	Output.RandSequence.SampleSeed  = Data.RandSeqSampleSeed;
	Output.Radiance = Data.Radiance;
	Output.Alpha = Data.Alpha;
	Output.PackedAlbedoNormal = Data.PackedAlbedoNormal;
	Output.Ray.Origin = Data.RayOrigin;
	Output.Ray.Direction = Data.RayDirection;
	Output.Ray.TMin = 0;
	Output.Ray.TMax = RAY_DEFAULT_T_MAX;
	Output.PathThroughput = Data.PathThroughput;
	Output.PackedRoughnessSigma = Data.PackedRoughnessSigma;
	Output.PackedScatterPhase = Data.PackedScatterPhase;
	return Output;
}
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
void StorePathStateData(FPathState PathState, uint Index)
{
	FPathTracingPackedPathState Data;
	Data.RandSeqSampleIndex = PathState.RandSequence.SampleIndex;
	Data.RandSeqSampleSeed  = PathState.RandSequence.SampleSeed;
	Data.Radiance = PathState.Radiance;
	Data.Alpha = PathState.Alpha;
	Data.PackedAlbedoNormal = PathState.PackedAlbedoNormal;
	Data.RayOrigin = PathState.Ray.Origin;
	Data.RayDirection = PathState.Ray.Direction;
	Data.PathThroughput = PathState.PathThroughput;
	Data.PackedRoughnessSigma = PathState.PackedRoughnessSigma;
	Data.PackedScatterPhase = PathState.PackedScatterPhase;
	PathStateData[Index] = Data;
}
#endif

RAY_TRACING_ENTRY_RAYGEN(PathTracingMainRG)
{
	uint2 DispatchIdx = DispatchRaysIndex().xy;
	const uint2 DispatchDim = DispatchRaysDimensions().xy;

	uint LinearPixelIndex = DispatchIdx.x + DispatchDim.x * DispatchIdx.y;

#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	const int FirstBounce = 0;
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
	if (LinearPixelIndex == 0)
	{
		// write other dimensions for indirect dispatch of the next bounce
		NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1) + 1] = 1;
		NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1) + 2] = 1;
	}
#endif

#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	uint PathIndex = DispatchIdx.x + 65536 * DispatchIdx.y;
#if PATH_TRACER_USE_ADAPTIVE_SAMPLING
	if (Iteration > 0)
	{
		if (LinearPixelIndex >= NumPathStates[0])
		{
			return; // nothing left to do on this thread
		}
		PathIndex = ActivePaths[LinearPixelIndex];
		DispatchIdx = uint2(PathIndex & 65535, PathIndex >> 16);
		LinearPixelIndex = DispatchIdx.x + ScanlineWidth * DispatchIdx.y;
	}
#endif
	FPathState PathState = CreatePathState(ComputePixelIndex(DispatchIdx),  ComputeTextureIndex(DispatchIdx));
#else // PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_INNER
	// compare to number of active paths from the previous bounce
	if (LinearPixelIndex >= NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * FirstBounce + 0])
	{
		return; // nothing left to do on this thread
	}
	const uint PathIndex = ActivePaths[LinearPixelIndex];
	FPathState PathState = LoadPathStateData((PathIndex & 65535) + ScanlineWidth * (PathIndex >> 16));
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
	const bool KeepGoing = PathTracingKernel(PathState, FirstBounce);
	const uint2 TextureIndex = ComputeTextureIndex(uint2(PathIndex & 65535, PathIndex >> 16));
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	{
		// Camera ray must write depth right away because we don't want to have to carry DepthZ in the PackedPathState
		PathState.WriteDepth(TextureIndex);
	}
#endif
	if (KeepGoing)
	{
	    // Add the path to the set of valid paths for the next bounce
		uint PathStateIndex;
		InterlockedAdd(NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1)], 1, PathStateIndex);
		NextActivePaths[PathStateIndex] = PathIndex;
		StorePathStateData(PathState, (PathIndex & 65535) + ScanlineWidth * (PathIndex >> 16));
	}
	else
	{
		// nothing left to do
		// Accumulate radiance and update pixel variance
		PathState.WritePixel(TextureIndex);
	}
#else // PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_TAIL
	for (int Bounce = FirstBounce; Bounce <= MaxBounces; Bounce++)
	{
		if (!PathTracingKernel(PathState, Bounce))
		{
			// kernel had nothing more to do
			break;
		}
	}

	// Accumulate radiance and update pixel variance
	const uint2 TextureIndex = ComputeTextureIndex(uint2(PathIndex & 65535, PathIndex >> 16));
	PathState.WritePixel(TextureIndex);
#endif
}
