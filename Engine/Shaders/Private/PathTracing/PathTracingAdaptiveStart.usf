// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "PathTracingAdaptiveUtils.ush"


RWBuffer<uint> NextActivePaths;
RWBuffer<uint> NumPathStates;
int2 TileTextureOffset;
uint2 DispatchDim;


[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingAdaptiveStartCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 DispatchIdx = DispatchThreadId;
	if (any(DispatchIdx >= DispatchDim))
	{
		return;
	}

	uint PathIndex = DispatchIdx.x + 65536 * DispatchIdx.y;
	if (PathIndex == 0)
	{
		// write other dimensions for indirect dispatch
		NumPathStates[1] = 1;
		NumPathStates[2] = 1;
	}

	float SampleScaleFactor = GetAdaptiveSampleFactor(DispatchIdx + TileTextureOffset);
	if (SampleScaleFactor > 1.0)
	{
		// This pixel needs more samples, allocate a path state for it
		uint PathStateIndex;
		InterlockedAdd(NumPathStates[0], 1, PathStateIndex);
		NextActivePaths[PathStateIndex] = PathIndex;
	}
}
