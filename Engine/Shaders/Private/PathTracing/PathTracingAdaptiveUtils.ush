// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

Texture2D<float4>   VarianceTexture;
SamplerState 		VarianceSampler;
uint3				VarianceTextureDims;
float 				AdaptiveSamplingErrorThreshold;


// Variance of the tonemapped color can be predicted from the variance of the scene linear color:
// https://en.wikipedia.org/wiki/Taylor_expansions_for_the_moments_of_functions_of_random_variables
float PerceptualError(float Mean, float StdErr)
{
	// The UE tone mapping curve is quite a bit more involved, so getting its derivative would be messy and likely overkill
	// instead just use a roughly S-curved shape curve that roughly does something filmic without costing too much
	float x = Mean;
#if 0
	// https://knarkowicz.wordpress.com/2016/01/06/aces-filmic-tone-mapping-curve/
	// saturate((x*(a*x+b))/(x*(c*x+d)+e))
	float a = 2.51f;
	float b = 0.03f;
	float c = 2.43f;
	float d = 0.59f;
	float e = 0.14f;
	return StdErr * (a * x * (2 * e + d * x) + b * (e - c * x * x)) / Square(x * (c * x + d) + e);
#else
	// Cheap fit to curve above
	// tonemap(x) = (6*x/(6*x+1))^2
	return StdErr * 72.0 * x * rcp(Pow3(6.0 * x + 1.0));
#endif
}

float4 SampleVarianceTexture(float2 VarianceUV, int Mip)
{
#if 0
	// cheap, but has artifacts 
	return VarianceTexture.SampleLevel(VarianceSampler, VarianceUV, Mip);
#else
	// Efficient GPU-Based Texture Interpolation using Uniform B-Splines
	// Daniel Ruijters, Bart M. ter Haar Romeny, Paul Suetens
	// Journal of Graphics GPU and Game Tools · January 2008
	const float2 TextureSize = float2(VarianceTextureDims.xy >> Mip);
	const float2 UV = VarianceUV * TextureSize - 0.5;
	const float2 BaseUV = floor(UV);
	const float2 F = UV - BaseUV;
	const float2 O = 1.0 - F; 
	const float2 F2 = F * F;
	const float2 O2 = O * O;
	const float2 W0 = (1.0f / 6.0f) * O2 * O;;
	const float2 W1 = (2.0f / 3.0f) - F2 * (1.0f - 0.5f * F);
	const float2 W2 = (2.0f / 3.0f) - O2 * (1.0f - 0.5f * O);
	const float2 W3 = (1.0f / 6.0f) * F2 * F;
	const float2 G0 = W0 + W1;
	const float2 G1 = W2 + W3;
	const float2 H0 = (W1 / G0 + BaseUV - 0.5) / TextureSize;
	const float2 H1 = (W3 / G1 + BaseUV + 1.5) / TextureSize;
	const float4 Tex00 = VarianceTexture.SampleLevel(VarianceSampler, float2(H0.x, H0.y), Mip);
	const float4 Tex10 = VarianceTexture.SampleLevel(VarianceSampler, float2(H1.x, H0.y), Mip);
	const float4 Tex01 = VarianceTexture.SampleLevel(VarianceSampler, float2(H0.x, H1.y), Mip);
	const float4 Tex11 = VarianceTexture.SampleLevel(VarianceSampler, float2(H1.x, H1.y), Mip);
	return lerp(lerp(Tex11, Tex10, G0.y),
				lerp(Tex01, Tex00, G0.y),
				G0.x);
#endif
}

float GetAdaptiveSampleFactor(uint2 TexCoord)
{
	float SampleScaleFactor = 0, SampleScale = 1;
	for (int Mip = 0, NumMips = VarianceTextureDims.z; Mip < NumMips; Mip++, SampleScale *= 4)
	{
		const int R = 2;
		for (int dy = -R; dy <= R; dy++)
		for (int dx = -R; dx <= R; dx++)
		{
			int2 Coord = (TexCoord >> Mip) + int2(dx, dy);
			if (all(uint2(Coord) < uint2(VarianceTextureDims.xy >> Mip)))
			{
				float4 Variance = VarianceTexture.Load(int3(Coord, Mip));
				float NumSamples = Variance.z;
				float OutVariance = abs(Variance.y - Variance.x * Variance.x);
				float Lum = Variance.x;
				float StdErr = sqrt(OutVariance / NumSamples);
				float RelErr = PerceptualError(Lum, StdErr);
				// project the multiple of how many more samples we will need
				SampleScaleFactor = max(SampleScaleFactor, (1 + Square(RelErr / AdaptiveSamplingErrorThreshold)) / NumSamples * SampleScale);
			}
		}
	}
	return SampleScaleFactor;
}
