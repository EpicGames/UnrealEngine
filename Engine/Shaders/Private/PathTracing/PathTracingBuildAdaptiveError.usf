// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

int2 InputResolution;
int2 OutputResolution;
SamplerState InputMipSampler;
Texture2D<float4>   InputMip;
RWTexture2D<float4> OutputMip;

float4 CombinePixels(float4 A, float4 B, float W)
{
	float NumSamples = lerp(A.z, B.z, W);
	float RatioA = A.z / NumSamples;
	float RatioB = B.z / NumSamples;
	return float4(lerp(A.xy * RatioA, B.xy * RatioB, W), NumSamples, 0);
}

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingBuildAdaptiveErrorTextureCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	int2 OutPixelCoord = int2(DispatchThreadId);
	if (any(OutPixelCoord >= OutputResolution))
	{
		return;
	}

	float4 Result = 0;
	const bool bOddX = (InputResolution.x & 1) == 1;
	const bool bOddY = (InputResolution.y & 1) == 1;

	if (bOddX)
	{
		if (bOddY)
		{
			// odd-odd, need 3x3 weighted downsample
			float W0x = float(OutputResolution.x - OutPixelCoord.x) / float(InputResolution.x);
			float W1x = float(OutputResolution.x                  ) / float(InputResolution.x);
			float W2x = float(                 1 + OutPixelCoord.x) / float(InputResolution.x);

			float W0y = float(OutputResolution.y - OutPixelCoord.y) / float(InputResolution.y);
			float W1y = float(OutputResolution.y                  ) / float(InputResolution.y);
			float W2y = float(                 1 + OutPixelCoord.y) / float(InputResolution.y);

			float4 Pixelx0 = 0;
			float4 Pixelx1 = 0;
			float4 Pixelx2 = 0;

			float4 Pixel00 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 0), 0));
			float4 Pixel10 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 0), 0));
			float4 Pixel20 = InputMip.Load(int3(2 * OutPixelCoord + int2(2, 0), 0));
			Pixelx0 = CombinePixels(Pixel00, Pixel10, W0x / (W0x + W1x));
			Pixelx0 = CombinePixels(Pixelx0, Pixel20, W2x);

			float4 Pixel01 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 1), 0));
			float4 Pixel11 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 1), 0));
			float4 Pixel21 = InputMip.Load(int3(2 * OutPixelCoord + int2(2, 1), 0));
			Pixelx1 = CombinePixels(Pixel01, Pixel11, W0x / (W0x + W1x));
			Pixelx1 = CombinePixels(Pixelx1, Pixel21, W2x);

			float4 Pixel02 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 2), 0));
			float4 Pixel12 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 2), 0));
			float4 Pixel22 = InputMip.Load(int3(2 * OutPixelCoord + int2(2, 2), 0));
			Pixelx2 = CombinePixels(Pixel02, Pixel12, W0x / (W0x + W1x));
			Pixelx2 = CombinePixels(Pixelx2, Pixel22, W2x);

			Result = CombinePixels(Pixelx0, Pixelx1, W0y / (W0y + W1y));
			Result = CombinePixels(Result, Pixelx2, W2y);
		}
		else
		{
			// odd-even 3x2 weighted downsample
			float W0x = float(OutputResolution.x - OutPixelCoord.x) / float(InputResolution.x);
			float W1x = float(OutputResolution.x                  ) / float(InputResolution.x);
			float W2x = float(                 1 + OutPixelCoord.x) / float(InputResolution.x);

			float4 Pixelx0 = 0;
			float4 Pixelx1 = 0;

			float4 Pixel00 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 0), 0));
			float4 Pixel10 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 0), 0));
			float4 Pixel20 = InputMip.Load(int3(2 * OutPixelCoord + int2(2, 0), 0));
			Pixelx0 = CombinePixels(Pixel00, Pixel10, W0x / (W0x + W1x));
			Pixelx0 = CombinePixels(Pixelx0, Pixel20, W2x);

			float4 Pixel01 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 1), 0));
			float4 Pixel11 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 1), 0));
			float4 Pixel21 = InputMip.Load(int3(2 * OutPixelCoord + int2(2, 1), 0));
			Pixelx1 = CombinePixels(Pixel01, Pixel11, W0x / (W0x + W1x));
			Pixelx1 = CombinePixels(Pixelx1, Pixel21, W2x);

			Result = CombinePixels(Pixelx0, Pixelx1, 0.5);
		}
	}
	else
	{
		if (bOddY)
		{
			// even-odd, need 2x3 weighted downsample
			float W0y = float(OutputResolution.y - OutPixelCoord.y) / float(InputResolution.y);
			float W1y = float(OutputResolution.y                  ) / float(InputResolution.y);
			float W2y = float(                 1 + OutPixelCoord.y) / float(InputResolution.y);

			float4 Pixelx0 = 0;
			float4 Pixelx1 = 0;
			float4 Pixelx2 = 0;

			float4 Pixel00 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 0), 0));
			float4 Pixel10 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 0), 0));
			Pixelx0 = CombinePixels(Pixel00, Pixel10, 0.5);

			float4 Pixel01 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 1), 0));
			float4 Pixel11 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 1), 0));
			Pixelx1 = CombinePixels(Pixel01, Pixel11, 0.5);

			float4 Pixel02 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 2), 0));
			float4 Pixel12 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 2), 0));
			Pixelx2 = CombinePixels(Pixel02, Pixel12, 0.5);

			Result = CombinePixels(Pixelx0, Pixelx1, W0y / (W0y + W1y));
			Result = CombinePixels(Result, Pixelx2, W2y);
		}
		else
		{
			// even-even, need 2x2 weighted downsample
			float4 Pixel00 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 0), 0));
			float4 Pixel10 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 0), 0));
			float4 Pixelx0 = CombinePixels(Pixel00, Pixel10, 0.5);

			float4 Pixel01 = InputMip.Load(int3(2 * OutPixelCoord + int2(0, 1), 0));
			float4 Pixel11 = InputMip.Load(int3(2 * OutPixelCoord + int2(1, 1), 0));
			float4 Pixelx1 = CombinePixels(Pixel01, Pixel11, 0.5);

			Result = CombinePixels(Pixelx0, Pixelx1, 0.5);
		}
	}
	Result.z *= 4.0; // even though we did a weighted average, pretend like we have 4x more samples now
	OutputMip[OutPixelCoord] = Result;
}