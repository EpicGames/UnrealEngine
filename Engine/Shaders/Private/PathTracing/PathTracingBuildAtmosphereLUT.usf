// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SkyAtmosphereCommon.ush"
#include "./Volume/PathTracingAtmosphereCommon.ush"

uint NumSamples;
uint Resolution;
RWTexture2D<float3> AtmosphereOpticalDepthLUT;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingBuildAtmosphereOpticalDepthLUTCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId >= Resolution))
	{
		return;
	}

	const float2 UV = (DispatchThreadId + 0.5) / float(Resolution);

	const float T = Atmosphere.TopRadiusKm;
	const float B = Atmosphere.BottomRadiusKm;

	const float2 Result = FromAtmosphereOpticalDepthLUTUV(UV, B, T);

	const float h = Result.x;
	const float c = Result.y;

	const float Rb = B / h;
	const float Rt = T / h;
	const float deltaT = Rt * Rt + (c * c - 1.0);
	const float deltaB = Rb * Rb + (c * c - 1.0);
	const float rsT = (-c + sqrt(max(deltaT, 0.0))); // distance to top atmosphere (farthest)
	const float rsB0 = (-c - sqrt(max(deltaB, 0.0))); // distance to ground (nearest)
	const float rsB1 = (-c + sqrt(max(deltaB, 0.0))); // distance to ground (farthest)

	float3 Tau = 0.0;
	if (deltaB > 0.0 && c < 0.0)
	{
		// we see the ground, integrate the variable portions numerically, and the part through the planet center analytically (since it is homogeneous)
		// this greatly increases the precision of the integral, reducing artifacts at rendertime
		const int NS0 = NumSamples / 2;
		const int NS1 = NumSamples - NS0;
		{
			// part above ground
			const float drs = rsB0 / float(NS0);
			const float ds = drs * h;
			for (int i = 0; i < NS0; i++)
			{
				const float rs = (i + 0.5) * drs;
				const float ViewHeightS = h * sqrt(1.0 + rs * rs + 2 * c * rs);
				Tau += SampleAtmosphereMediumRGB(float3(0, 0, ViewHeightS)).Extinction * ds;
			}
		}
		// planet core: homogeneous, so optical depth is simple
		Tau += h * (rsB1 - rsB0) * SampleAtmosphereMediumRGB(float3(0, 0, 0)).Extinction;
		{
			// part on the other side of the planet
			const float drs = (rsT - rsB1) / float(NS1);
			const float ds = drs * h;
			for (int i = 0; i < NS1; i++)
			{
				const float rs = rsB1 + (i + 0.5) * drs;
				const float ViewHeightS = h * sqrt(1.0 + rs * rs + 2 * c * rs);
				Tau += SampleAtmosphereMediumRGB(float3(0, 0, ViewHeightS)).Extinction * ds;
			}
		}
	}
	else
	{
		// we see the sky only, single integral
		const int NS = NumSamples;
		const float drs = rsT / float(NS);
		const float ds = drs * h;
		for (int i = 0; i < NS; i++)
		{
			const float rs = (i + 0.5) * drs;
			const float ViewHeightS = h * sqrt(1.0 + rs * rs + 2 * c * rs);
			Tau += SampleAtmosphereMediumRGB(float3(0, 0, ViewHeightS)).Extinction * ds;
		}
	}

	// store the integrated density (also known as optical depth)
	// note that this is a unitless quantity. We store this rather than transmittance so we can have the most numerical precision to perform subtractions along the ray
	AtmosphereOpticalDepthLUT[DispatchThreadId] = Tau;
}
