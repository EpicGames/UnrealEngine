// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "../SkyAtmosphereCommon.ush"
#include "./Volume/PathTracingAtmosphereCommon.ush"
#include "./Volume/PathTracingCloudsCommon.ush"
#include "./Volume/PathTracingCloudsMaterialCommon.ush"
#include "./Utilities/PathTracingRandomSequence.ush"

uint NumSamples;
uint TemporalSeed;
uint Iteration;

RWTexture2D<float4> CloudAccelerationMap;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingBuildCloudAccelerationMapCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId >= CloudAccelMapResolution))
	{
		return;
	}

	ResolvedView = ResolveView();

	// get min and max altitude of the clouds
	const float T = CloudLayerTopKm;
	const float B = CloudLayerBotKm;

	// tracing view distance (around the origin -- can't go further than the top radius)
	const float D = CloudClipDistKm;
	
	float MinDensity = POSITIVE_INFINITY;
	float MaxDensity = 0;
	float ZMin = POSITIVE_INFINITY;
	float ZMax = NEGATIVE_INFINITY;
	if (Iteration > 0)
	{
		float4 Data = CloudAccelerationMap[DispatchThreadId];
		MaxDensity = Data.x;
		MinDensity = Data.y;
		ZMin = Data.z;
		ZMax = Data.w;
	}
	for (uint Sample = 0; Sample < NumSamples; Sample++)
	{
		float3 Jitter = RandomSequenceCreate(DispatchThreadId.x + DispatchThreadId.y * 65536, Iteration * NumSamples + Sample).Get3D();
		const float2 UV = 2.0 * ((DispatchThreadId + Jitter.xy) / float(CloudAccelMapResolution)) - 1.0;
		// Project a 2d map onto the surface of a sphere
		float3 ClipPos;
		ClipPos.xy = D * UV;
		float R2 = length2(ClipPos.xy);
		float TopZ = sqrt(max(T * T - R2, 0.0));
		float BotZ = sqrt(max(B * B - R2, 0.0));
		ClipPos.z = lerp(BotZ, TopZ, Jitter.z);

		float Z = ClipPos.z - CloudClipRadiusKm; // remember height of the point (before rotation, relative to center)
		float H = length(ClipPos);

		float3 WorldPos = mul(ClipPos, GetCloudClipBasis()) + CloudClipCenterKm; // TangentToWorld
		FDFVector3 AbsoluteWorldPosition = DFMultiply(WorldPos, SKY_UNIT_TO_CM);
		// run the material and encode the result, which includes the effect that quantization will have during rendering
		FSampleCloudMaterialResult Result = SampleCloudMaterial(AbsoluteWorldPosition, H, B, T);
		
		float Density   = MATVEC_Max(Result.Extinction);
		float DensityLo = MATVEC_Min(Result.Extinction);

		// If shader had no density but some emission, we need to artificially boost the density so that we get at least _some_ raymarch steps in the empty regions
		if (Density == 0 && any(Result.Emission > 0))
		{
			Density = CloudInvVoxelWidth; // ~1 step per voxel
		}

		if (Density > 0)
		{
			ZMin = min(ZMin, Z);
			ZMax = max(ZMax, Z);
		}
		MaxDensity = max(MaxDensity, Density);
		MinDensity = min(MinDensity, DensityLo);
	}
	CloudAccelerationMap[DispatchThreadId] = float4(MaxDensity, MinDensity, ZMin, ZMax);
}
