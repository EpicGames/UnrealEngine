// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingTypes.h"
#include "/Engine/Shared/PathTracingDefinitions.h"
#include "../IntersectionUtils.ush"

#define USE_ATTENUATION_TERM 0
#define USE_IES_TERM 0
#include "Light/PathTracingLightCommon.ush"

uint SceneInfiniteLightCount;
float3 SceneLightsTranslatedBoundMin;
float3 SceneLightsTranslatedBoundMax;
RWTexture2DArray<uint> RWLightGrid;
RWBuffer<uint> RWLightGridData;
uint LightGridResolution;
uint LightGridMaxCount;
uint LightGridAxis;

#define RECT_TEST 2 // 0: half-space only
                    // 1: barndoors (planes only)
					// 2: barndoors+aabb culling

#define CONE_TEST 1 // 0: test against cone AABB
					// 1: accurate cone/aabb test

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingBuildLightGridCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId.xy >= LightGridResolution) || DispatchThreadId.z >= 3)
	{
		return;
	}
	// figure out dimension of the 3d grid and the current ID to be filled
	uint3 VoxelId = 0, VoxelRes = 1;
	int Axis = DispatchThreadId.z;
	if (LightGridAxis != Axis && LightGridAxis < 3)
	{
		RWLightGrid[DispatchThreadId] = 0;
		return;
	}
	switch (Axis)
	{
		case 0: VoxelId.yz = DispatchThreadId.xy; VoxelRes.yz = LightGridResolution; break;
		case 1: VoxelId.xz = DispatchThreadId.xy; VoxelRes.xz = LightGridResolution; break;
		case 2: VoxelId.xy = DispatchThreadId.xy; VoxelRes.xy = LightGridResolution; break;
	}
	// get bounding box of current voxel
	float3 Lo = lerp(SceneLightsTranslatedBoundMin, SceneLightsTranslatedBoundMax, float3(VoxelId + 0) / float3(VoxelRes));
	float3 Hi = lerp(SceneLightsTranslatedBoundMin, SceneLightsTranslatedBoundMax, float3(VoxelId + 1) / float3(VoxelRes));

	uint NumVisible = 0;
	uint MaxVisible = min(LightGridMaxCount, RAY_TRACING_LIGHT_COUNT_MAXIMUM - SceneInfiniteLightCount);
	uint Offset = LightGridMaxCount * (Axis + 3 * (DispatchThreadId.x + LightGridResolution * DispatchThreadId.y));
	bool AllSingular = true;
	for (uint LightIndex = SceneInfiniteLightCount; LightIndex < SceneLightCount; LightIndex++)
	{
		uint LightType = SceneLights[LightIndex].Flags & PATHTRACER_FLAG_TYPE_MASK;
		float3 Center = GetTranslatedPosition(LightIndex);
		float Radius = 1.0 / GetAttenuation(LightIndex);
		float3 Normal = GetNormal(LightIndex);

		if (!AABBOverlapsSphere(Lo, Hi, Center, Radius))
		{
			// bounds don't overlap there sphere of influence
			continue;
		}

		// reject against normal
		if (LightType != PATHTRACING_LIGHT_POINT)
		{
			if (IsBoxBehindPlane(Lo, Hi, Normal, Center))
			{
				continue;
			}
		}
#if RECT_TEST > 0
		if (LightType == PATHTRACING_LIGHT_RECT)
		{
			// check barndoors
			const float BarnCos = GetRectLightBarnCosAngle(LightIndex);
			if (BarnCos > 0.035)
			{
				const float BarnLen = GetRectLightBarnLength(LightIndex);
				const float HalfWidth = GetWidth(LightIndex) * 0.5;
				const float HalfHeight = GetHeight(LightIndex) * 0.5;

				if (!AABBOverlapsRectLightFrustum(Lo, Hi, Center, Normal, GetdPdv(LightIndex), HalfWidth, HalfHeight, Radius, BarnCos, BarnLen, RECT_TEST == 2))
				{
					continue;
				}
			}
		}
#endif
		if (LightType == PATHTRACING_LIGHT_SPOT)
		{
			const float CosOuter = GetCosConeAngles(LightIndex).x;
			if (!AABBOverlapsCurvedCone(Lo, Hi, Center, Normal, CosOuter, Radius, CONE_TEST == 0))
			{
				continue;
			}
		}

		{
			AllSingular = AllSingular && all(SceneLights[LightIndex].Dimensions == 0);
			RWLightGridData[Offset + NumVisible] = LightIndex;
			NumVisible++;
			// TODO: handle overflow better?
			if (NumVisible >= MaxVisible)
			{
				break;
			}
		}
	}
	if (AllSingular)
	{
		NumVisible |= PATHTRACER_LIGHT_GRID_SINGULAR_MASK;
	}
	RWLightGrid[DispatchThreadId] = NumVisible;
}
