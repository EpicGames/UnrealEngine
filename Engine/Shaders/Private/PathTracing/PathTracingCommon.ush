// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PathTracingCommon.ush: path tracing payload structures
=============================================================================*/

#pragma once

#include "Material/PathTracingMaterialCommon.ush"
#include "Utilities/PathTracingRandomSequence.ush"
#include "Utilities/PathTracingRIS.ush"
#include "../RayTracing/RayTracingCommon.ush"
#include "../OctahedralCommon.ush"
#include "../DoubleFloat.ush"
#include "/Engine/Shared/PathTracingDefinitions.h"

// These flags are analogous to RAY_TRACING_PAYLOAD_* flags. There are currently 9 bits available.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE                   (1 << 0) // Indicates that ray has hit the front face of a primitive. Set by closest hit shader.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED_MATERIAL           (1 << 1) // Indicates that ray has hit a primitive surface with a two sided material. Set by closest hit shader. Used by GPU Lightmass to detect invalid surfaces.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT                   (1 << 2) // Indicates that ray has hit a primitive which should be held out from alpha.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_MASK_SHIFT    (3)
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_COLOR         (1 << 3) // Indicates that ray has hit a primitive that receives decals color. Set by closest hit shader.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_NORMAL        (1 << 4) // Indicates that ray has hit a primitive that receives decals normal. Set by closest hit shader.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_ROUGHNESS     (1 << 5) // Indicates that ray has hit a primitive that receives decals metallic/specular/roughness. Set by closest hit shader.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_USE_DBUFFER_LOOKUP           (1 << 6) // Indicates that ray has hit a primitive that uses DBufferTextureLookup. Set by closest hit shader.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_OUTOUT_DEPTH				  (1 << 7) // Indicates that ray has hit a primitive that needs to output depth. Set by closest hit shader and used in primary ray only.
#define PATH_TRACING_PAYLOAD_OUTPUT_FLAG_UNUSED8                      (1 << 8) // Free for future use

#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK                 (7 << 0) // Reserve 3 bits for ray-mask
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_CAMERA                0       // Indicates that this is a camera ray which can trigger a number of special behaviors in the closest hit shader
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_SHADOW                1       // Indicates that this is a path tracer visibility ray (which supports material evaluation for transparent shadows and fake caustics)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_DIFFUSE      2
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_SPECULAR     3
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_VOLUME       4
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH               (1 << 4) // Indicates the payload contains a scene depth value (exclusive with dbuffer usage)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT            (1 << 5) // Indicates this ray should not process any translucent geometry (so we can skip them in our opaque traces)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_INFO_MASK               (3 << 6) // Reserve 2 bits for debug-info (ask material to give us back some debug information)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_TRIANGLE_ID             (1 << 6)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_INSTANCE_ID             (2 << 6)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_CLUSTER_ID              (3 << 6)
#define PATH_TRACING_PAYLOAD_INPUT_FLAG_UNUSED8                       (1 << 8) // Free for future use

#define PATH_TRACING_PAYLOAD_NORMAL_BITS  		32
#define PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS 16 						// Could be dropped to 12 or even 10 if we need more room in the future


// The Path Tracer does not directly follow the SHADINGMODELID_* defines because it classifies models slightly differently and maps legacy materials (pre-substrate) onto their substrate equivalents
#define PATH_TRACING_SHADINGMODELID_BASIC     	0 // basic slab with SSS
#define PATH_TRACING_SHADINGMODELID_FOLIAGE   	1 // basic slab with diffuse transmission lobe
#define PATH_TRACING_SHADINGMODELID_CLEARCOAT 	2 // basic slab where the second specular lobe is layered over the first instead of mixed (legacy clearcoat)
#define PATH_TRACING_SHADINGMODELID_SOLID_GLASS 3 // slab with refraction
#define PATH_TRACING_SHADINGMODELID_THIN_GLASS 	4 // slab with thin-refraction
#define PATH_TRACING_SHADINGMODELID_EYE         5 // special purpose model for eyes
#define PATH_TRACING_SHADINGMODELID_HAIR        6 // special purpose model for hair
#define PATH_TRACING_SHADINGMODELID_MEDIUM      7 // represent the result of scattering in a volume (phase function)
#define PATH_TRACING_SHADINGMODELID_VOLUMETRIC  8 // surface shading model for particles that collects light from all directions
#define PATH_TRACING_SHADINGMODELID_LAMBERT     9 // pure lambert response (used for lighting after SSS random walk)
#define PATH_TRACING_SHADINGMODELID_MASK        0xf // 4 bits reserved


#define PATH_TRACING_ABSORPTION_SCALE  0.01        // inverse distance at which we hit the transmission color (Beer's law)

// Ignore hair strands if the ray origin is beyond this radius. This is meant to workaround a performance cliff that can occur on some hardware.
// In ordinary scenes, the most likely occurance of this problem is when combining hair and volumetric atmosphere as rays from several Km can travel back towards the hair and cause long delays or even TDRs
#define PATH_TRACING_SKIP_HAIR_DISTANCE      1e6

uint GetPathTracingHairRayFlags(float3 RayOrigin)
{
    return length2(RayOrigin) < Pow2(PATH_TRACING_SKIP_HAIR_DISTANCE) ? 0 : RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES;
}

#define SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE	(SUBSTRATE_ENABLED && PLATFORM_ENABLES_SUBSTRATE_SPECULAR_PROFILE)
#define SUBSTRATE_PATH_TRACING_GLINTS 			(SUBSTRATE_ENABLED && PLATFORM_ENABLES_SUBSTRATE_GLINTS)


// Given a ray with origin O in direction D, assuming we have found a valid hit at THit, compute a new TMin value that we can use to retrace
// the same ray and find more hits. This is used for transparency, mesh decals and SSS tracing.
float ComputeNewTMin(float3 O, float3 D, float THit)
{
#if NEED_TMIN_WORKAROUND
	// Some hardware shifts the ray origin to TMin which can lead to erroneous hits against TMin when shifted back
	// This code tries to find to smallest increment that causes the shifted origin to change
	uint Inc = 1;
	float TMin = asfloat(asuint(THit) + Inc);
	while (all((mad(D, TMin, O) == mad(D, THit, O)))) {
		Inc *= 2;
		TMin = asfloat(asuint(THit) + Inc);
	}
	return TMin;
#else
	// On conformant hardware, the following is sufficient to avoid the previous hit
	return asfloat(asuint(THit) + 1);
#endif
}


// Utilities to help weight the contribution of the albedo/normal to the denoising buffers
// We want cases like low roughness mirrors to "pass through" the signal
static const float DenoiserMaxRoughness = 0.15f; // don't bother past this level
static const float DenoiserMinRoughness = 0.10f; // capture anything below this level

// Given the current path roughness, how much contribution should the current hit make to the denoiser AOVs?
float DenoiserAOVWeight(float PathRoughness)
{
	return 1 - saturate((PathRoughness - DenoiserMinRoughness) / (DenoiserMaxRoughness - DenoiserMinRoughness));
}
// For a given surface of roughness R, how much should its own contribution be counted
// This heuristic lets us skip the contribution a mirror's albedo would normally make because we will be seeing "through" it to the next hit
float DenoiserRoughnessWeight(float Roughness, float PathRoughness)
{
	float MinRoughness = max(DenoiserMinRoughness, PathRoughness);
	return saturate((Roughness - MinRoughness) / (DenoiserMaxRoughness - MinRoughness));
}

// This function is meant for the solid glass case so we can store the transmittance as a color in [0,1]
float3 PathTracingGlassTransmittanceToExtinction(float3 TransmittanceColor)
{
	return -log(max(TransmittanceColor, 1e-8f)) * PATH_TRACING_ABSORPTION_SCALE;
}

uint PayloadEncodeUnitVector(float3 Normal)
{
	const int BITS = PATH_TRACING_PAYLOAD_NORMAL_BITS;
	const float Scale = float((1u << (BITS / 2)) - 2) / 2; // largest representable even number (so we can encode 0)
	float2 OctF = UnitVectorToOctahedron(Normal);
	int2 Oct = int2(round(OctF * Scale + Scale));
	return Oct.x + (1u << (BITS / 2)) * Oct.y;
}

float3 PayloadDecodeUnitVector(uint Encoded)
{
	const int BITS = PATH_TRACING_PAYLOAD_NORMAL_BITS;
	const uint Mask = (1u << (BITS / 2)) - 1;
	const float Scale = float((1u << (BITS / 2)) - 2);

	int2 Oct = int2(Encoded & Mask, (Encoded >> (BITS / 2)) & Mask);
	float2 OctF = saturate(float2(Oct) / Scale);
	return OctahedronToUnitVector(2 * OctF - 1);
}

// Encode a tangent vector relative to a given Normal
uint PayloadEncodeTangentAngle(float3 Normal, float3 Tangent)
{
	const uint Mask = (1u << (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 2)) - 1;
	const uint SignShiftX = 31 - (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 2);
	const uint SignShiftY = 31 - (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 1);
	const uint SignMaskX = 0x80000000u >> SignShiftX;
	const uint SignMaskY = 0x80000000u >> SignShiftY;
	const float Scale = float(Mask);

	float3x3 Basis = GetTangentBasis(Normal);
	float X = dot(Basis[0], Tangent);
	float Y = dot(Basis[1], Tangent);
	// NOTE: We use 'Y' to define the angle so that if Tangent happens to be (0,0,0) it will be decoded back as the "x" axis in the reference basis
	float PseudoAngle = abs(Y) / (abs(X) + abs(Y)); // Coarse ArcTan, returns value in [0,1]
	uint QuantizedBits = uint(round(PseudoAngle * Scale));
	// Merge with sign bits of X and Y
	return QuantizedBits | ((asuint(X) >> SignShiftX) & SignMaskX) | ((asuint(Y) >> SignShiftY) & SignMaskY);
}

// Decode a tangent vector relative to a given Normal
float3 PayloadDecodeTangentAngle(float3 Normal, uint Bits)
{
	const uint Mask = (1u << (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 2)) - 1;
	const uint SignShiftX = 31 - (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 2);
	const uint SignShiftY = 31 - (PATH_TRACING_PAYLOAD_TANGENT_ANGLE_BITS - 1);
	const uint SignMaskX = 0x80000000u >> SignShiftX;
	const uint SignMaskY = 0x80000000u >> SignShiftY;
	const float Scale = float(Mask);

	float3x3 Basis = GetTangentBasis(Normal);
	float Y = float(Bits & Mask) / Scale;
	float2 P = float2(1.0f - Y, Y);
	// restore sign bits
	P.x = asfloat(asuint(P.x) | ((Bits & SignMaskX) << SignShiftX));
	P.y = asfloat(asuint(P.y) | ((Bits & SignMaskY) << SignShiftY));
	P = normalize(P);
	return Basis[0] * P.x + Basis[1] * P.y;
}

uint PayloadEncodeHDRColor(float3 rgb)
{
	return PackRGB998E6(rgb);
}

float3 PayloadDecodeHDRColor(uint rgb)
{
	return UnpackRGB998E6(rgb);
}

uint PayloadEncodeLDRColor(float3 rgb)
{
	return PackRGB111110(rgb);
}

float3 PayloadDecodeLDRColor(uint rgb)
{
	return UnpackRGB111110(rgb);
}

uint PayloadEncodeRoughnessAniso(float3 RoughnessData, float Anisotropy)
{
	uint r = uint(floor(saturate(RoughnessData.x) * 255.0 + 0.5));
	uint g = uint(floor(saturate(RoughnessData.y) * 255.0 + 0.5));
	uint b = uint(floor(saturate(RoughnessData.z) * 255.0 + 0.5));
	uint a = uint(floor(clamp(Anisotropy, -1.0, 1.0) * 127.0 + 127.0)); // Range is [-1,1] so use an SNorm encoding here so we can represent 0.0 exactly
	return (r << 24) | (g << 16) | (b << 8) | a;
}

float4 PayloadDecodeRoughnessAniso(uint rgba)
{
	float r = float((rgba >> 24)      ) * (1.0 / 255.0);
	float g = float((rgba >> 16) & 255) * (1.0 / 255.0);
	float b = float((rgba >>  8) & 255) * (1.0 / 255.0);
	float a = float((rgba      ) & 255) * (1.0 / 127.0) - 1.0;
	return float4(r, g, b, a);
}

uint PayloadEncodeFuzz(float FuzzAmount, float FuzzRoughness)
{
	// TODO: revisit this as we could pack more data here
	return PackFloat2ToUInt(FuzzAmount, FuzzRoughness);
}

void PayloadDecodeFuzz(uint Bits, inout float FuzzAmount, inout float FuzzRoughness)
{
	float2 Data = UnpackFloat2FromUInt(Bits);
	FuzzAmount = Data.x;
	FuzzRoughness = Data.y;
}

float3 AdjustShadingNormal(float3 ShadingNormal, float3 GeoNormal, float3 V_World)
{
	// Clip shading normal in a view dependent way such that the reflection stays above the geometric normal
	// This introduces a bit of view-dependency to the shading normal but fixes dark artifacts around grazing angles

	float3 D = -V_World;
	float3 R = reflect(D, ShadingNormal);

	// https://iquilezles.org/www/articles/dontflip/dontflip.htm
	float k = dot(R, GeoNormal);
	if (k < 0.0)
	{
		return normalize(normalize(R - k * GeoNormal) - D);
	}
	return ShadingNormal;
}


#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_PATH_TRACING_MATERIAL) && IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
#error "Path tracing and GPULightmass payloads are mutually exclusive. They should not be both enabled at once."
#endif

// This payload structure is what we transport between RGS/CHS/AHS programs
struct FPackedPathTracingPayload : FMinimalPayload
{
	// float FMinimalPayload::HitT                       // 4  bytes
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	uint PackedData[7];                                  // 28 bytes (encoded data, depends on shading model and ray type)
	// 32 bytes total
#else
	uint PackedData[18];                                 // 72 bytes
	// 76 bytes total
#endif

	float UnpackBSDFOpacity() { return f16tof32(PackedData[0] & 0xFFFF); }
	uint GetPathTracingShadingModelID() { return (PackedData[0] >> 16) & PATH_TRACING_SHADINGMODELID_MASK; }
	uint GetFlags() { return (PackedData[0] >> 20) & 0x1FF; }
	void SetFlag(uint Flag) { PackedData[0] |= Flag << 20; }
	void RemoveFlag(uint Flag) { PackedData[0] &= ~(Flag << 20); }
	uint GetPrimitiveLightingChannelMask() { return (PackedData[0] >> 29) & 0x7; }

	// Flag methods
	void SetCameraRay() { SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_CAMERA); }
	bool IsCameraRay() { return (GetFlags() & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_CAMERA; }

	void SetVisibilityRay() { SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_SHADOW); }
	bool IsVisibilityRay() { return (GetFlags() & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_SHADOW; }

	bool IsFrontFace() { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }
	
	bool IsDecalReceiver() 
	{
		const uint Mask = 
			PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_COLOR |
			PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_NORMAL | 
			PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_ROUGHNESS | 
			PATH_TRACING_PAYLOAD_OUTPUT_FLAG_USE_DBUFFER_LOOKUP;
		return (GetFlags() & Mask) != 0;
	}
	bool HasDecalResponseColor()     { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_COLOR    ) != 0; }
	bool HasDecalResponseNormal()    { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_NORMAL   ) != 0; }
	bool HasDecalResponseRoughness() { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_ROUGHNESS) != 0; }
	bool UsesDBufferLookup()         { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_USE_DBUFFER_LOOKUP      ) != 0; }

	void SetOutputDepth() { SetFlag(PATH_TRACING_PAYLOAD_OUTPUT_FLAG_OUTOUT_DEPTH); }
	bool ShouldOutputDepth() { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_OUTOUT_DEPTH) != 0; }

	bool IsHoldout() { return (GetFlags() & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT) != 0; }

	// These method are meant to be used only when IsVisibilityRay() is true
	float3 GetRayThroughput()
	{
		return float3(
			asfloat(PackedData[1]),
			asfloat(PackedData[2]),
			asfloat(PackedData[3])
		);
	}

	void SetRayThroughput(float3 RayThroughput)
	{
		PackedData[1] = asuint(RayThroughput.x);
		PackedData[2] = asuint(RayThroughput.y);
		PackedData[3] = asuint(RayThroughput.z);
	}

	float GetPathRoughness()
	{
		return asfloat(PackedData[4]);
	}

	void SetPathRoughness(float PathRoughness)
	{
		PackedData[4] = asuint(PathRoughness);
	}

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	// NOTE: GPU Lightmass doesnt't have a shading model that needs volumetric absorption along shadow rays
#else
	// tau: optical depth along shadow ray, this is the integral of the extinction coefficient
	float3 GetTau()
	{
		return float3(
			asfloat(PackedData[5]),
			asfloat(PackedData[6]),
			asfloat(PackedData[7])
		);
	}

	void SetTau(float3 Tau)
	{
		PackedData[5] = asuint(Tau.x);
		PackedData[6] = asuint(Tau.y);
		PackedData[7] = asuint(Tau.z);
	}
#endif

	// Check if we have already run the same hit through AHS
	bool CheckDuplicateAHS(float T)
	{
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
		return T == HitT || T == asfloat(PackedData[5]);
#else
		// either equal to the last hit or second-to-last hit
		return T == HitT || T == asfloat(PackedData[8]);
#endif
	}

	void UpdateDuplicateAHS(float T)
	{
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
		PackedData[5] = asuint(HitT); // remember second to last
#else
		PackedData[8] = asuint(HitT); // remember second to last
#endif
		HitT = T; // update most recent hit
	}


#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	float3 UnpackWorldGeoNormal()    { return PayloadDecodeUnitVector(PackedData[1]); }
	float3 UnpackWorldNormal()       { return PayloadDecodeUnitVector(PackedData[2]); }
	float3 UnpackRadiance()          { return PayloadDecodeHDRColor(PackedData[3]); }
	float3 UnpackBaseColor()         { return PayloadDecodeLDRColor(PackedData[4]); }
	float3 UnpackTransparencyColor() { return PayloadDecodeLDRColor(PackedData[5]); }
	float3 UnpackSubsurfaceColor()   { return PayloadDecodeLDRColor(PackedData[6]); }


#else

	float3 UnpackWorldGeoNormal() 	 { return PayloadDecodeUnitVector(PackedData[1]); }
	float3 UnpackWorldSmoothNormal() { return PayloadDecodeUnitVector(PackedData[2]); }
	float3 UnpackWorldNormal()       { return PayloadDecodeUnitVector(PackedData[3]); }
	float3 UnpackRadiance()          { return PayloadDecodeHDRColor(PackedData[4]); }

	void PackWorldNormal(float3 N)   { PackedData[3] = PayloadEncodeUnitVector(N); }
	void PackRadiance(float3 R)      { PackedData[4] = PayloadEncodeHDRColor(R); }

	float3 UnpackWorldTangent()       { return PayloadDecodeTangentAngle(UnpackWorldNormal(), PackedData[5]); }
	float UnpackIor()                 { return f16tof32(PackedData[5] >> 16); }
	
	float3 UnpackDiffuseColor()       { return PayloadDecodeLDRColor(PackedData[6]); }
	float3 UnpackSpecularColor()      { return PayloadDecodeLDRColor(PackedData[7]); }
	float3 UnpackSpecularEdgeColor()  { return PayloadDecodeLDRColor(PackedData[8]); }
	float4 UnpackRoughnessAniso()     { return PayloadDecodeRoughnessAniso(PackedData[9]); }
	float3 UnpackTransparencyColor()  { return PayloadDecodeLDRColor(PackedData[10]); }
	float3 UnpackMeanFreePath()       { return float3(f16tof32(PackedData[11]),
	                                                  f16tof32(PackedData[11] >> 16),
	                                                  f16tof32(PackedData[12])); }
	float UnpackPhaseG()              { return f16tof32(PackedData[12] >> 16); }
	float3 UnpackFuzzColor()          { return PayloadDecodeLDRColor(PackedData[13]); }
	float3 UnpackWeightV()            { return PayloadDecodeLDRColor(PackedData[15]); }
	float3 UnpackTransmittanceN()     { return PayloadDecodeLDRColor(PackedData[16]); }

	float UnpackCoverageAboveAlongN() { return abs(f16tof32(PackedData[17] >> 16)); }
	float UnpackSpecularProfileId()   { return f16tof32(PackedData[17]); }

	void PackDiffuseColor(float3 C)   { PackedData[6] = PayloadEncodeLDRColor(C); }
	void PackSpecularColor(float3 C)  { PackedData[7] = PayloadEncodeLDRColor(C); }
	void PackRoughnessAniso(float4 D) { PackedData[9] = PayloadEncodeRoughnessAniso(D.xyz, D.w); }

	float4 GetDBufferA()
	{
		return float4(
			UnpackFloat2FromUInt(PackedData[8]),
			UnpackFloat2FromUInt(PackedData[9])
		);
	}

	void SetDBufferA(float4 DBufferA)
	{
		PackedData[8] = PackFloat2ToUInt(DBufferA.xy);
		PackedData[9] = PackFloat2ToUInt(DBufferA.zw);
	}

	float4 GetDBufferB()
	{
		return float4(
			UnpackFloat2FromUInt(PackedData[10]),
			UnpackFloat2FromUInt(PackedData[11])
		);
	}

	void SetDBufferB(float4 DBufferB)
	{
		PackedData[10] = PackFloat2ToUInt(DBufferB.xy);
		PackedData[11] = PackFloat2ToUInt(DBufferB.zw);
	}

	float4 GetDBufferC()
	{
		return float4(
			UnpackFloat2FromUInt(PackedData[12]),
			UnpackFloat2FromUInt(PackedData[13])
		);
	}

	void SetDBufferC(float4 DBufferC)
	{
		PackedData[12] = PackFloat2ToUInt(DBufferC.xy);
		PackedData[13] = PackFloat2ToUInt(DBufferC.zw);
	}
	
	void SetStochasticSlabRand(float SlabRand)
	{
		PackedData[14] = asuint(SlabRand);
	}
	
	float GetStochasticSlabRand()
	{
		return asfloat(PackedData[14]);
	}

	float GetSceneDepth()
	{
		// NOTE: This is aliased with GetDBufferA, but this is ok as the two features may not be used together
		return asfloat(PackedData[8]);
	}

	void SetSceneDepth(float SceneZ)
	{
		PackedData[8] = asuint(SceneZ);
	}

#if PATH_TRACER_USE_CLOUD_SHADER
	// Volumetric callable shaders
	void SetVolumetricCallableShaderInput(float3 Origin, float3 Direction, float TMin, float TMax, FRandomSequence RandSeq, uint Flags, float3 Throughput, float CloudDensity)
	{
		HitT = TMin;
		PackedData[0]  = asuint(TMax);
		PackedData[1]  = asuint(Origin.x);
		PackedData[2]  = asuint(Origin.y);
		PackedData[3]  = asuint(Origin.z);
		PackedData[4]  = asuint(Direction.x);
		PackedData[5]  = asuint(Direction.y);
		PackedData[6]  = asuint(Direction.z);
		PackedData[7]  = RandSeq.SampleIndex;
		PackedData[8]  = RandSeq.SampleSeed;
		PackedData[9]  = (Flags & 0xFFFF) | (f32tof16(CloudDensity) << 16);
		PackedData[10] = asuint(Throughput.x);
		PackedData[11] = asuint(Throughput.y);
		PackedData[12] = asuint(Throughput.z);
	}
	
	void SetVolumetricCallableShaderInputCloudFactor(float CloudFactor)
	{
		PackedData[13] = asuint(CloudFactor);
	}

	void SetVolumetricCallableShaderRISContext(FRISContext SurfaceContext)
	{
		PackedData[13] = asuint(SurfaceContext.WeightSum);
		PackedData[14] = asuint(SurfaceContext.RandSample);
	}

	FRayDesc GetVolumetricCallableShaderInputRay()
	{
		FRayDesc Ray;
		Ray.Origin    = asfloat(uint3(PackedData[1], PackedData[2], PackedData[3]));
		Ray.Direction = asfloat(uint3(PackedData[4], PackedData[5], PackedData[6]));
		Ray.TMin      = HitT;
		Ray.TMax      = asfloat(PackedData[0]);
		return Ray;
	}

	FRandomSequence GetVolumetricCallableShaderInputRandSequence()
	{
		FRandomSequence RandSeq;
		RandSeq.SampleIndex = PackedData[7];
		RandSeq.SampleSeed  = PackedData[8];
		return RandSeq;
	}

	uint GetVolumetricCallableShaderInputFlags()
	{
		return PackedData[9] & 0xFFFF;
	}

	float3 GetVolumetricCallableShaderInputThroughput()
	{
		return asfloat(uint3(PackedData[10], PackedData[11], PackedData[12]));
	}

	float GetVolumetricCallableShaderInputCloudFactor()
	{
		return asfloat(PackedData[13]);
	}

	float GetVolumetricCallableShaderInputCloudDensity()
	{
		return f16tof32(PackedData[9] >> 16);
	}

	FRISContext GetVolumetricCallableShaderRISContext()
	{
		FRISContext Context;
		Context.WeightSum  = asfloat(PackedData[13]);
		Context.RandSample = asfloat(PackedData[14]);
		return Context;
	}

	void SetVolumetricCallableShaderOutput(FRandomSequence RandSeq, float3 Throughput, float3 Albedo)
	{
		PackedData[0] = RandSeq.SampleIndex;
		PackedData[1] = RandSeq.SampleSeed;
		// sampling mode: need to pack transmittance and albedo together
		PackedData[2] = PackFloat2ToUInt(Throughput.xy);
		PackedData[3] = PackFloat2ToUInt(float2(Throughput.z, Albedo.x));
		PackedData[4] = PackFloat2ToUInt(Albedo.yz);
	}

	void SetVolumetricCallableShaderOutput(FRandomSequence RandSeq, float3 Throughput)
	{
		// Increased precision for Transmittance mode
		PackedData[0] = RandSeq.SampleIndex;
		PackedData[1] = RandSeq.SampleSeed;
		PackedData[2] = asuint(Throughput.x);
		PackedData[3] = asuint(Throughput.y);
		PackedData[4] = asuint(Throughput.z);
	}

	void SetVolumetricCallableShaderOutputSample(float SampleT, float3 AlbedoRayleigh, float3 AlbedoHG1, float PhaseG, float3 AlbedoHG2, float3 DualPhaseData, float3 PayloadThroughput)
	{
		HitT = SampleT;
		PackedData[5]  = PackFloat2ToUInt(PayloadThroughput.xy);
		PackedData[6]  = PackFloat2ToUInt(float2(PayloadThroughput.z, PhaseG));
		PackedData[7]  = PayloadEncodeLDRColor(AlbedoRayleigh);
		PackedData[8]  = PayloadEncodeLDRColor(AlbedoHG1);
		PackedData[9]  = PayloadEncodeLDRColor(AlbedoHG2);
		PackedData[10] = PayloadEncodeLDRColor(DualPhaseData * float3(0.5, 0.5, 1.0) + float3(0.5, 0.5, 0.0));
	}

	void SetVolumetricCallableShaderOutputCloudDensityFactor(float Factor)
	{
		PackedData[12] &= 0xFFFFu; // clear upper bits
		PackedData[12] |= f32tof16(Factor) << 16;
	}

	void SetVolumetricCallableShaderOutputRadiance(float3 Radiance, float Alpha)
	{
		PackedData[11] = PayloadEncodeHDRColor(Radiance);
		PackedData[12] &= 0xFFFF0000u; // clear lower bits
		PackedData[12] |= f32tof16(Alpha);
	}

	FRandomSequence GetVolumetricCallableShaderOutputRandSeq()
	{
		FRandomSequence RandSeq;
		RandSeq.SampleIndex = PackedData[0];
		RandSeq.SampleSeed  = PackedData[1];
		return RandSeq;
	}

	float3 GetVolumetricCallableShaderOutputThroughput()
	{
		return float3(UnpackFloat2FromUInt(PackedData[2]), UnpackFloat2FromUInt(PackedData[3]).x);
	}

	float3 GetVolumetricCallableShaderOutputAlbedo()
	{
		return float3(UnpackFloat2FromUInt(PackedData[3]).y, UnpackFloat2FromUInt(PackedData[4]));
	}

	float3 GetVolumetricCallableShaderOutputTransmittanceThroughput()
	{
		// Increased precision for Transmittance mode
		return asfloat(uint3(PackedData[2], PackedData[3], PackedData[4]));
	}

	float3 GetVolumetricCallableShaderOutputPayloadThroughput()
	{
		return float3(UnpackFloat2FromUInt(PackedData[5]), UnpackFloat2FromUInt(PackedData[6]).x);
	}

	float3 GetVolumetricCallableShaderOutputRayleighWeight()
	{
		return PayloadDecodeLDRColor(PackedData[7]);
	}

	float4 GetVolumetricCallableShaderOutputHG()
	{
		return float4(PayloadDecodeLDRColor(PackedData[8]), UnpackFloat2FromUInt(PackedData[6]).y);
	}

	float3 GetVolumetricCallableShaderOutputDualHGWeight()
	{
		return PayloadDecodeLDRColor(PackedData[9]);
	}

	float3 GetVolumetricCallableShaderOutputDualHGPhaseData()
	{
		return PayloadDecodeLDRColor(PackedData[10]) * float3(2, 2, 1) - float3(1, 1, 0);
	}

	float3 GetVolumetricCallableShaderOutputRadiance()
	{
		return PayloadDecodeHDRColor(PackedData[11]);
	}

	float GetVolumetricCallableShaderOutputAlpha()
	{
		return f16tof32(PackedData[12]);
	}

	float GetVolumetricCallableShaderOutputCloudDensityFactor()
	{
		return f16tof32(PackedData[12] >> 16);
	}

#endif // PATH_TRACER_USE_CLOUD_SHADER

#endif // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
};

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_PATH_TRACING_MATERIAL)
CHECK_RT_PAYLOAD_SIZE(FPackedPathTracingPayload)
#endif
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
CHECK_RT_PAYLOAD_SIZE(FPackedPathTracingPayload)
#endif

// This payload structure is the expanded version of the above which is more convenient to work with
struct FPathTracingPayload : FMinimalPayload
{
	float3 Radiance;
	float3 WorldGeoNormal;           // normal of the actual triangle (faceted)
	float3 WorldNormal;              // normal output of the material (includes normal/bump)
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	// skip smooth normal for GPULightmass
#else
	float3 WorldSmoothNormal;        // smooth normal before normal/bump is applied
#endif

	float  BSDFOpacity;              // how much should we weigh down the bsdf by? (this also accounts for the probability of picking a particular lobe in multi-lobed materials)
	float3 TransparencyColor;        // how much do we see straight through the surface?
	uint   PathTracingShadingModelID;
	uint   Flags;
	uint   PrimitiveLightingChannelMask;

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	float3 BaseColor;
	float3 SubsurfaceColor;
#else
	float3 DiffuseColor;
	float3 SpecularColor;
	float3 SpecularEdgeColor;
	float3 RoughnessData; // r0,r1,rblend
	float  Anisotropy;
	float3 WorldTangent;
	float3 MeanFreePath;
	float PhaseG;
	float Ior;
	float FuzzAmount;
	float FuzzRoughness;
	float3 FuzzColor;
	float3 WeightV;
	float3 TransmittanceN;
	float CoverageAboveAlongN;
#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
	float SpecularProfileId;
#endif
#if SUBSTRATE_PATH_TRACING_GLINTS
	float GlintValue;
	float2 GlintUV;
	float2 GlintUVdx;
	float2 GlintUVdy;
#endif
	// not transported through packed payload, but used in raygen
	float3 SubsurfaceColor;

#endif // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)

	
	float3 TranslatedWorldPos;

	void SetFrontFace() { Flags |= PATH_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE; }
	bool IsFrontFace() { return (Flags & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }

	void SetDecalReceiver(uint DecalResponseMask) { Flags |= (DecalResponseMask & 0x07) << PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_MASK_SHIFT; }

	void SetUseDBufferLookup() { Flags |= PATH_TRACING_PAYLOAD_OUTPUT_FLAG_USE_DBUFFER_LOOKUP; }
	bool UsesDBufferLookup() { return (Flags & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_USE_DBUFFER_LOOKUP) != 0; }
	
	void SetHoldout() { Flags |= PATH_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT; }
	bool IsHoldout() { return (Flags & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT) != 0; }

	void SetMaterialTwoSided() { Flags |= PATH_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED_MATERIAL; }
	bool IsMaterialTwoSided() { return (Flags & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED_MATERIAL) != 0; }

	void SetOutputDepth() { Flags |= PATH_TRACING_PAYLOAD_OUTPUT_FLAG_OUTOUT_DEPTH; }
	bool ShouldOutputDepth() { return Flags & PATH_TRACING_PAYLOAD_OUTPUT_FLAG_OUTOUT_DEPTH; }

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	float3 GetBaseColor() { return BaseColor; }
	float3 GetSubsurfaceColor() { return SubsurfaceColor; }
	bool IsMaterialTransmissive()
	{
		return PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_FOLIAGE;
	}
#else // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)

	// SHADINGMODELID_SUBSTRATE
	float3 GetMaxLobeWeight()
	{
		return WeightV * lerp(1.0f, TransmittanceN, CoverageAboveAlongN); // largest value LobeWeight() could return
	}

	// SHADINGMODELID_LAMBERT and SHADINGMODELID_MEDIUM
	float3 GetBaseColor() { return DiffuseColor; }
	void SetBaseColor(float3 BaseColor) { DiffuseColor = BaseColor; }

	// SHADINGMODELID_MEDIUM
	float3 GetHGWeight() { return SpecularColor; }
	float GetHGPhaseG() { return PhaseG; }
	void SetHG(float3 Weight, float G)
	{
		SpecularColor = Weight;
		PhaseG = G;
	}
	float3 GetDualHGWeight() { return SpecularEdgeColor; }
	float3 GetDualHGPhaseData() { return RoughnessData; }
	void SetDualHG(float3 Weight, float3 PhaseData)
	{
		SpecularEdgeColor = Weight;
		RoughnessData = PhaseData;
	}
	float GetCloudFactor() { return Anisotropy; }
	void SetCloudFactor(float Factor) { Anisotropy = Factor; }

	// SHADINGMODELID_EYE
	float GetEyeRoughness() { return RoughnessData.x; }
	void SetEyeRoughness(float R) { RoughnessData.x = R; }
	float3 GetEyeCausticNormal() { return normalize(2.0 * FuzzColor - 1.0); } // SUBSTRATE_TODO: not optimal since packed as color
	void SetEyeCausticNormal(float3 CausticNormal) { FuzzColor = 0.5 * CausticNormal + 0.5; }
	float GetEyeIrisMask() { return Anisotropy; }
	void SetEyeIrisMask(float IrisMask) { Anisotropy = IrisMask; }
	float3 GetEyeIrisNormal() { return normalize(2.0 * SpecularEdgeColor - 1.0); }  // SUBSTRATE_TODO: not optimal since packed as color
	void SetEyeIrisNormal(float3 IrisNormal) { SpecularEdgeColor = IrisNormal * 0.5 + 0.5; }

	// SHADINGMODELID_HAIR
	float GetHairLongitudinalRoughness() { return RoughnessData.x; }
	void SetHairLongitudinalRoughness(float Roughness) { RoughnessData.x = Roughness;}
	float GetHairAzimuthalRoughness() { return RoughnessData.y; }
	void SetHairAzimuthalRoughness(float AzimuthalRoughness) { RoughnessData.y = AzimuthalRoughness; }
	float GetHairSpecular() { return RoughnessData.z; }
	void SetHairSpecular(float Specular) { RoughnessData.z = Specular; }
	float2 GetHairPrimitiveUV() { return SpecularEdgeColor.xy; }
	void SetHairPrimitiveUV(float2 UV) { SpecularEdgeColor.xy = UV; }

	// SHADINGMODELID_SOLID_GLASS
	float3 GetExtinction() { return rcp(MeanFreePath); }
	float3 GetGlassAlbedo() { return DiffuseColor; }
	void SetGlassAlbedo(float3 A) { DiffuseColor = A; }
	float GetGlassPhase() { return PhaseG; }
	void SetGlassPhase(float G) { PhaseG = G; }

	// SHADINGMODELID_THIN_TRANSLUCENT
	float3 GetTransmittanceColor() { return MeanFreePath; }
	void SetTransmittanceColor(float3 TransmittanceColor) { MeanFreePath = TransmittanceColor; }

	// SHADINGMODELID_TWOSIDED_FOLIAGE
	float3 GetFoliageTransmissionColor() { return MeanFreePath; }
	void SetFoliageTransmissionColor(float3 C) { MeanFreePath = C; }

	// SHADINGMODELID_CLEAR_COAT
	float3 GetBottomCoatNormal() { return MeanFreePath; }
	void SetBottomCoatNormal(float3 BottomNormal) { MeanFreePath = BottomNormal; }

	// Methods used by the integrator

	bool HasRefraction()
	{
		return Ior > 0.0;
	}

	bool IsMaterialSolidGlass()
	{
		return PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_SOLID_GLASS;
	}

	bool IsMaterialThinGlass()
	{
		return PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_THIN_GLASS && HasRefraction();
	}

	bool IsMaterialTransmissive()
	{
		return
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_FOLIAGE ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_HAIR ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_MEDIUM ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_VOLUMETRIC ||
			IsMaterialSolidGlass() ||
			IsMaterialThinGlass();
	}

	bool IsSubsurfaceMaterial()
	{
		return
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_BASIC ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE;
	}

	// Are the Fuzz* parameters used for the fuzz lobe?
	bool HasFuzzLobe()
	{
		return
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_BASIC ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_FOLIAGE ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_CLEARCOAT ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_SOLID_GLASS ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_THIN_GLASS ||
			PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE;
	}

#endif // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
};

FPackedPathTracingPayload InitPathTracingPayload(uint ScatterType, float PathRoughness)
{
	FPackedPathTracingPayload Output = (FPackedPathTracingPayload)0;
	Output.SetPathRoughness(PathRoughness);
	Output.SetMiss();
	switch (ScatterType)
	{
		case PATHTRACER_SCATTER_DIFFUSE:   Output.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_DIFFUSE);  break;
		case PATHTRACER_SCATTER_SPECULAR:  
		case PATHTRACER_SCATTER_REFRACT:   Output.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_SPECULAR); break;
		case PATHTRACER_SCATTER_VOLUME:    Output.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_VOLUME);   break;
		case PATHTRACER_SCATTER_CAMERA: 
		default:                           Output.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_CAMERA); break;
	}
	return Output;
}

FPackedPathTracingPayload InitPathTracingVisibilityPayload(float PathRoughness)
{
	FPackedPathTracingPayload Output = (FPackedPathTracingPayload)0;
	// Signal to the AHS we want to evaluate the opacity
	// The payload is used to carry the path throughput (for transparent shadows)
	// and current path roughness (for approximate caustics)
	Output.SetVisibilityRay();
	Output.SetPathRoughness(PathRoughness);
	Output.SetRayThroughput(1.0);
	// NOTE: We start with HitT = 0 which counts as "IsHit()". This will be changed by the miss shader if we don't register any hits (or the AHS writes don't accumulate all the way to 0)
	// This is because we want to be able to skip the CHS on shadow rays. Therefore the miss shader is the one that will be responsible for telling us a ray did not hit anything opaque.
	// If after tracing the ray we still have IsHit() returning true, this means we must have hit something opaque (without having run either CHS or AHS).
	// This also has the benefit of preparing the "mailbox" used for detecting duplicate invocations of AHS with an invalid value.
	Output.HitT = 0.0;
	return Output;
}

FPackedPathTracingPayload PackPathTracingPayload(FPathTracingPayload Input)
{
	FPackedPathTracingPayload Output = (FPackedPathTracingPayload)0;
	Output.HitT = Input.HitT;

	Output.PackedData[0]  = f32tof16(Input.BSDFOpacity);                                      			// 16 bits
	Output.PackedData[0] |= (Input.PathTracingShadingModelID & PATH_TRACING_SHADINGMODELID_MASK) << 16;	// 4  bits
	Output.PackedData[0] |= (Input.Flags & 0x1FF) << 20;                                      			// 9  bits
	Output.PackedData[0] |= (Input.PrimitiveLightingChannelMask & 0x7) << 29;                 			// 3  bits
	                                                                                          			// 32 bits total
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	Output.PackedData[1] = PayloadEncodeUnitVector(Input.WorldGeoNormal);
	Output.PackedData[2] = PayloadEncodeUnitVector(Input.WorldNormal);
	Output.PackedData[3] = PayloadEncodeHDRColor(Input.Radiance);
	Output.PackedData[4] = PayloadEncodeLDRColor(Input.BaseColor);
	Output.PackedData[5] = PayloadEncodeLDRColor(Input.TransparencyColor);
	Output.PackedData[6] = PayloadEncodeLDRColor(Input.SubsurfaceColor);
#else
	Output.PackedData[1] = PayloadEncodeUnitVector(Input.WorldGeoNormal);
	Output.PackedData[2] = PayloadEncodeUnitVector(Input.WorldSmoothNormal);
	Output.PackedData[3] = PayloadEncodeUnitVector(Input.WorldNormal);
	Output.PackedData[4] = PayloadEncodeHDRColor(Input.Radiance);
	Output.PackedData[5] = PayloadEncodeTangentAngle(Output.UnpackWorldNormal(), Input.WorldTangent);
	Output.PackedData[5]|= f32tof16(Input.Ior) << 16;

	Output.PackedData[6] = PayloadEncodeLDRColor(Input.DiffuseColor);
	Output.PackedData[7] = PayloadEncodeLDRColor(Input.SpecularColor);
	Output.PackedData[8] = PayloadEncodeLDRColor(Input.SpecularEdgeColor);
	Output.PackedData[9] = PayloadEncodeRoughnessAniso(Input.RoughnessData, Input.Anisotropy);
	Output.PackedData[10] = PayloadEncodeLDRColor(Input.TransparencyColor);
	Output.PackedData[11] = PackFloat2ToUInt(float2(Input.MeanFreePath.xy));
	Output.PackedData[12] = PackFloat2ToUInt(float2(Input.MeanFreePath.z, Input.PhaseG));
	Output.PackedData[13] = PayloadEncodeLDRColor(Input.FuzzColor);
	Output.PackedData[14] = PayloadEncodeFuzz(Input.FuzzAmount, Input.FuzzRoughness);
	Output.PackedData[15] = PayloadEncodeLDRColor(Input.WeightV);
	Output.PackedData[16] = PayloadEncodeLDRColor(Input.TransmittanceN);
#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
	float SpecularProfileId = Input.SpecularProfileId;
#else
	float SpecularProfileId = 0;
#endif
	Output.PackedData[17] = PackFloat2ToUInt(float2(SpecularProfileId, abs(Input.CoverageAboveAlongN)));

#if SUBSTRATE_PATH_TRACING_GLINTS
	// In order to not increase the data payload size, we alias SSS/Fuzz parameters when glint are enabled
	if (Input.GlintValue < 1)
	{
		const uint2 PackedGlintsData = PackGlints(Input.GlintValue, Input.GlintUV);
		Output.PackedData[11] = PackedGlintsData.x;
		Output.PackedData[12] = PackedGlintsData.y;
		Output.PackedData[13] = PackFloat2ToUInt(Input.GlintUVdx);
		Output.PackedData[14] = PackFloat2ToUInt(Input.GlintUVdx);
		Output.PackedData[17] |= 0x80000000u; // Encode presence of glints in sign bit of CoverageAboveAlongN
	}
#endif

#endif // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	return Output;
}

#if RAYGENSHADER
FPathTracingPayload UnpackPathTracingPayload(FPackedPathTracingPayload Input, FRayDesc Ray)
{
	FPathTracingPayload Output = (FPathTracingPayload)0;

	Output.HitT = Input.HitT;
	Output.TranslatedWorldPos = Ray.Origin + Output.HitT * Ray.Direction;

	Output.BSDFOpacity                  = Input.UnpackBSDFOpacity();
	Output.PathTracingShadingModelID    = Input.GetPathTracingShadingModelID();
	Output.Flags                        = Input.GetFlags();
	Output.PrimitiveLightingChannelMask = Input.GetPrimitiveLightingChannelMask();
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	Output.WorldGeoNormal    = Input.UnpackWorldGeoNormal();
	Output.WorldNormal       = Input.UnpackWorldNormal();
	Output.Radiance          = Input.UnpackRadiance();
	Output.BaseColor         = Input.UnpackBaseColor();
	Output.TransparencyColor = Input.UnpackTransparencyColor();
	Output.SubsurfaceColor   = Input.UnpackSubsurfaceColor();
#else
	Output.WorldGeoNormal    = Input.UnpackWorldGeoNormal();
	Output.WorldSmoothNormal = Input.UnpackWorldSmoothNormal();
	Output.WorldNormal       = Input.UnpackWorldNormal();
	Output.Radiance          = Input.UnpackRadiance();
	Output.WorldTangent      = Input.UnpackWorldTangent();
	Output.Ior               = Input.UnpackIor();

	Output.DiffuseColor        = Input.UnpackDiffuseColor();
	Output.SpecularColor       = Input.UnpackSpecularColor();
	Output.SpecularEdgeColor   = Input.UnpackSpecularEdgeColor();
	float4 RoughnessAniso      = Input.UnpackRoughnessAniso();
	Output.RoughnessData       = RoughnessAniso.xyz;
	Output.Anisotropy          = RoughnessAniso.w;
	Output.TransparencyColor   = Input.UnpackTransparencyColor();
	Output.MeanFreePath        = Input.UnpackMeanFreePath();
	Output.PhaseG              = Input.UnpackPhaseG();
	Output.FuzzColor           = Input.UnpackFuzzColor();
	PayloadDecodeFuzz(Input.PackedData[14], Output.FuzzAmount, Output.FuzzRoughness);
	Output.WeightV             = Input.UnpackWeightV();
	Output.TransmittanceN      = Input.UnpackTransmittanceN();
	Output.CoverageAboveAlongN = Input.UnpackCoverageAboveAlongN();
#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
	Output.SpecularProfileId   = Input.UnpackSpecularProfileId();
#endif
	
#if SUBSTRATE_PATH_TRACING_GLINTS
	// TODO: figure out how to make glint unpacking closer to functions above
	// In order to not increase the data payload size, we alias SSS/Fuzz parameters when glint are enabled
	const bool bHasGlints = Input.PackedData[17] >> 31u;
	if (bHasGlints)
	{
		Output.MeanFreePath.x      = 0.f;
		Output.MeanFreePath.y      = 0.f;
		Output.MeanFreePath.z      = 0.f;
		Output.PhaseG              = 0.f;
		Output.FuzzColor           = 0.f;
		Output.FuzzAmount          = 0.f;
		Output.FuzzRoughness       = 0.f;

		UnpackGlints(uint2(Input.PackedData[11], Input.PackedData[12]), Output.GlintValue, Output.GlintUV);
		Output.GlintUVdx = UnpackFloat2FromUInt(Input.PackedData[13]);
		Output.GlintUVdy = UnpackFloat2FromUInt(Input.PackedData[14]);
	}
	else
	{
		// When glints are not used, default to full density so we get the regular specular highlight
		Output.GlintValue = 1.0f;
	}
#endif // SUBSTRATE_PATH_TRACING_GLINTS
#endif // IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_GPULIGHTMASS)
	return Output;
}
#endif
