// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
ReferencePathTracinPixelShader.ush: Reference path tracing  compositing pixel shader for progressive update.
===============================================================================================*/

#include "../Common.ush"
#include "../ColorMap.ush"
#include "PathTracingAdaptiveUtils.ush"

uint Iteration;
uint MaxSamples;
int ProgressDisplayEnabled;
Texture2D<float4> RadianceTexture;
Texture2D<float> DepthTexture;
float PreExposure; // Taken from the view, but adjusted by the base exposure already applied to the Radiance texture
int AdaptiveSamplingVisualize;

void CompositeMain(
	in noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0,
	out float OutDepth : SV_DEPTH
)
{
	float2 UV = UVAndScreenPos.xy;
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	int3 TexCoord = int3(UV * BufferSize - View.ViewRectMin.xy, 0);

	float4 Radiance = RadianceTexture.Load(TexCoord);
	Radiance = select(IsFinite(Radiance), Radiance, MaxHalfFloat); // if any component overflowed half precision, clamp it now

	// NOTE: UE has the convention of using 1.0 - Alpha
	// We must saturate the result because the path tracer's alpha is stochastic in the presence of volumetrics
	// It is important that the clamp happens _after_ averaging, otherwise this can bias the estimate
	OutColor = float4(Radiance.rgb * PreExposure, saturate(1.0 - Radiance.a));

	if (AdaptiveSamplingVisualize == 1)
	{
		// Visualize which pixels are still being worked and color them according to how many more samples we think they need
		float2 VarianceUV = float2(TexCoord.xy + 0.5) * View.BufferSizeAndInvSize.zw;
		float SampleScaleFactor = GetAdaptiveSampleFactor(TexCoord.xy);
		if (SampleScaleFactor > 1.0)
		{
			float4 Variance = VarianceTexture.Load(TexCoord);
			float Ramp = Variance.z * SampleScaleFactor / MaxSamples;
			OutColor.xyz = ColorMapTurbo(Ramp);
		}
	}
	if (AdaptiveSamplingVisualize == 2)
	{
		// Visualize sample count relative to maximum
		float4 Variance = VarianceTexture.Load(TexCoord);
		float Ramp = Variance.z / min(Iteration + 1, MaxSamples);
		OutColor.xyz = ColorMapTurbo(Ramp);
	}

	if (AdaptiveSamplingVisualize >= 3 && AdaptiveSamplingVisualize < 3 + int(VarianceTextureDims.z))
	{
		//float4 Variance = VarianceTexture.Load(TexCoord);
		float2 VarianceUV = float2(TexCoord.xy + 0.5) / float2(VarianceTextureDims.xy);
		int Mip = AdaptiveSamplingVisualize - 3;
		float4 Variance = SampleVarianceTexture(VarianceUV, Mip);
		float NumSamples = Variance.z;
		float OutVariance = Variance.y - Variance.x * Variance.x;
		float Lum = Variance.x;
		float StdErr = sqrt(OutVariance / NumSamples);
		float RelErr = PerceptualError(Lum, StdErr);
		OutColor.xyz = RelErr;
	}
	
	if (ProgressDisplayEnabled && (Iteration + 1) < MaxSamples)
	{
		// Draw a progress meter for how far along we are to the max number of passes
		float Aspect = View.BufferSizeAndInvSize.y * View.BufferSizeAndInvSize.z;
		float Size = 50.0 * View.BufferSizeAndInvSize.z;
		float2 P = float2(UV.x, UV.y * Aspect);
		float2 RectMin = View.ViewRectMin.xy;
		float2 RectMax = View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy;

		// center progress in middle-bottom of the frame
		float2 Center = float2(lerp(RectMin.x, RectMax.x, 0.5), RectMax.y) * View.BufferSizeAndInvSize.zz - float2(0, Size * 0.75);
		P -= Center;
		float Fraction = saturate(float(Iteration + 1) / float(MaxSamples));
		// simple linear progress bar
		float2 ProgressSize = float2(2.0 * Size, 0.15 * Size);
		if (all(abs(P) < ProgressSize))
		{
			P += ProgressSize;
			P /= ProgressSize * 2;
			float Blend = smoothstep(Fraction - fwidth(P.x), Fraction + fwidth(P.x), P.x);
			// Grey/Red blend
			float3 ProgressBarColor = lerp(float3(1.0, 0.1, 0.1), float3(0.5, 0.5, 0.5), Blend);
			OutColor.xyz = lerp(OutColor.xyz, ProgressBarColor, 0.75);
		}
	}

	OutDepth = DepthTexture.Load(TexCoord);
}
