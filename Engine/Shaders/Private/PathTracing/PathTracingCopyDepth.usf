// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

Texture2D<float> DepthTexture;
float PreExposure; // Taken from the view, but adjusted by the base exposure already applied to the Radiance texture
int AdaptiveSamplingVisualize;

void CopyDepth(
	in noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float OutDepth : SV_DEPTH,
	out float4 OutColor : SV_Target0
)
{
	float2 UV = UVAndScreenPos.xy;
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	int3 TexCoord = int3(UV * BufferSize - View.ViewRectMin.xy, 0);
	OutDepth = DepthTexture.Load(TexCoord);
	OutColor = 0;
}
