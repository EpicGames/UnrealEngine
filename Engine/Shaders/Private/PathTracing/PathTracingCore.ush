// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define RANDSEQ_SFC_TEXTURE 1

#define USE_PATH_TRACING_LIGHT_GRID	1
#define USE_RAY_TRACING_DECAL_GRID	1

// Debug printing is enabled from config file with r.PathTracing.Debug
#ifndef DEBUG_ENABLE
#define DEBUG_ENABLE 0
#endif

#include "../HeterogeneousVolumes/HeterogeneousVolumesVoxelGridTypes.ush"
#include "../Common.ush"
#include "../PostProcessCommon.ush"
#include "../RectLight.ush"
#include "../RayTracing/RayTracingCommon.ush"

#include "PathTracingCommon.ush"
#include "../RayTracing/RayTracingHitGroupCommon.ush"
#include "PathTracingDecals.ush"

#include "../ShadingModels.ush"

#include "./Utilities/PathTracingRandomSequence.ush"
#include "./Utilities/PathTracingRIS.ush"
#include "./Light/PathTracingLightSampling.ush"
#include "./Light/PathTracingLightGrid.ush"
#include "./Material/PathTracingMaterialSampling.ush"
#include "./Volume/PathTracingVolume.ush"
#include "./Volume/PathTracingVolumeSampling.ush"

#if DEBUG_ENABLE
#include "../ShaderPrint.ush"
#endif

#if PATH_TRACER_USE_SER
#include "/Engine/Public/NvidiaExtensionsCommon.ush"
#endif


float BlendFactor;
float BaseExposure;
uint Iteration;
uint TemporalSeed;
uint MaxSamples;
uint MaxBounces;
uint MaxSSSBounces;
float SSSGuidingRatio;

float BackgroundAlpha;

// 0: only Material sampling
// 1: only Light sampling
// 2: both Material and Light
uint MISMode;

// 0: only Density sampling
// 1: Light sampling
uint VolumeMISMode;

float ApproximateCaustics; // Either 0 (off) or 1 (on) -- done this way to avoid conditionals
uint SamplerType;
uint ApplyDiffuseSpecularOverrides;
uint EnabledDirectLightingContributions;
uint EnabledIndirectLightingContributions;
float MaxPathIntensity;
float MaxNormalBias;
float FilterWidth;
float CameraFocusDistance;
float2 CameraLensRadius;
float Petzval;
float PetzvalFalloffPower;
float2 PetzvalExclusionBoxExtents;
float PetzvalExclusionBoxRadius;

RWTexture2D<float4> RadianceTexture;
#if PATH_TRACER_USE_ADAPTIVE_SAMPLING
RWTexture2D<float4> VarianceTexture;
#endif
RWTexture2D<float4> AlbedoTexture;
RWTexture2D<float4> NormalTexture;
RWTexture2D<float>  DepthTexture;
RaytracingAccelerationStructure TLAS;
uint SceneVisibleLightCount;

Buffer<float> StartingMediumData;

struct FPathState
{                                   // packed size
	FRandomSequence RandSequence;   //  8 bytes
	float3 Radiance;                // 12 bytes
	float Alpha;                   //  4 bytes
	uint3 PackedAlbedoNormal;       //  6 bytes // half precision (Albedo)
	//  6 bytes // half precision (Normal)
	FRayDesc Ray;                   // 12+12 = 24 bytes (TMin/TMax do not need to be stored)
	float3 PathThroughput;          // 12 bytes
	uint2 PackedRoughnessSigma;     // 2 bytes // half precision (PathRoughness)
	                                // 6 bytes // half precision  (SigmaT)
	                                // 3 bits - packed in sign bits of SigmaT (FirstScatterType)
	uint2 PackedScatterPhase;       // 6 bytes // half precision (SigmaS)
	                                // 2 bytes // phase g (could be quantized further)
	                                // 80 bytes total (see FPathTracingPackedPathState)

	// Temporary parameters not stored
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	float DepthZ;                   // Camera ray depth (don't need it otherwise)
#endif
#if DEBUG_ENABLE && PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	uint2 PixelCoord;               // Pixel index/coord
#endif

	float3 GetAlbedo()
	{
		return float3(UnpackFloat2FromUInt(PackedAlbedoNormal.x),
		              UnpackFloat2FromUInt(PackedAlbedoNormal.y).x);
	}

	float3 GetNormal()
	{
		return float3(UnpackFloat2FromUInt(PackedAlbedoNormal.y).y,
		              UnpackFloat2FromUInt(PackedAlbedoNormal.z));
	}

	void SetAlbedoNormal(float3 Albedo, float3 Normal)
	{
		PackedAlbedoNormal.x = PackFloat2ToUInt(Albedo.xy);
		PackedAlbedoNormal.y = PackFloat2ToUInt(float2(Albedo.z, Normal.x));
		PackedAlbedoNormal.z = PackFloat2ToUInt(Normal.yz);
	}

	uint GetFirstScatterType()
	{
		// Extract SigmaT sign bits
		uint a = (PackedRoughnessSigma.x >> 29) & 4u;
		uint b = (PackedRoughnessSigma.y >> 30) & 2u;
		uint c = (PackedRoughnessSigma.y >> 15) & 1u;
		return a | b | c;
	}

	void SetFirstScatterType(uint ScatterType)
	{
		uint a = ScatterType & 4u;
		uint b = ScatterType & 2u;
		uint c = ScatterType & 1u;
		// erase old value
		PackedRoughnessSigma.x &= 0x7fffffffu;
		PackedRoughnessSigma.y &= 0x7fff7fffu;
		// set new value
		PackedRoughnessSigma.x |= a << 29;
		PackedRoughnessSigma.y |= b << 30;
		PackedRoughnessSigma.y |= c << 15;
	}

	float GetPathRoughness()
	{
		return f16tof32(PackedRoughnessSigma.x);
	}

	void SetPathRoughness(float PathRoughness)
	{
		PackedRoughnessSigma.x &= 0xffff0000u;
		PackedRoughnessSigma.x |= f32tof16(PathRoughness);
	}
	
	float3 GetSigmaT()
	{
		return abs(float3(f16tof32(PackedRoughnessSigma.x >> 16),
		                  f16tof32(PackedRoughnessSigma.y),
		                  f16tof32(PackedRoughnessSigma.y >> 16)));
	}

	void SetSigmaT(float3 SigmaT)
	{
		// erase old value (keeping sign bits)
		PackedRoughnessSigma.x &= 0x8000ffffu;
		PackedRoughnessSigma.y &= 0x80008000u;
		SigmaT = abs(SigmaT); // make sure there are no sign bits
		// write new value
		PackedRoughnessSigma.x |= f32tof16(SigmaT.x) << 16;
		PackedRoughnessSigma.y |= f32tof16(SigmaT.y);
		PackedRoughnessSigma.y |= f32tof16(SigmaT.z) << 16;
	}

	float4 GetSigmaSAndPhase()
	{
		return float4(UnpackFloat2FromUInt(PackedScatterPhase.x), UnpackFloat2FromUInt(PackedScatterPhase.y));
	}

	void SetSigmaSAndPhase(float3 SigmaS, float PhaseG)
	{
		PackedScatterPhase.x = PackFloat2ToUInt(SigmaS.xy);
		PackedScatterPhase.y = PackFloat2ToUInt(float2(SigmaS.z, PhaseG));
	}

	bool HasMadeContributionScatter()
	{
		// The first scattering type that matters for the PATHTRACER_CONTRIBUTION_* flags is the first non-refractive one
		// We choose to track camera and pure refracted paths together so that objects behind glass can be treated the same as directly visible objects.
		uint FirstScatterType = GetFirstScatterType();
		return !(FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
			     FirstScatterType == PATHTRACER_SCATTER_REFRACT);
	}

	bool HasMadeDiffuseOrSpecularScatter()
	{
		uint FirstScatterType = GetFirstScatterType();
		return FirstScatterType == PATHTRACER_SCATTER_DIFFUSE || FirstScatterType == PATHTRACER_SCATTER_SPECULAR;
	}

    uint GetInstanceInclusionMaskOpaque()
    {
        uint FirstScatterType = GetFirstScatterType();
        if (FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
            FirstScatterType == PATHTRACER_SCATTER_REFRACT)
        {
            return PATHTRACER_MASK_CAMERA_OPAQUE;
        }
        if (FirstScatterType == PATHTRACER_SCATTER_SPECULAR)
        {
            return PATHTRACER_MASK_REFLECTION_OPAQUE;
        }
        return PATHTRACER_MASK_INDIRECT_OPAQUE;
    }

    uint GetInstanceInclusionMaskTranslucent()
    {
        uint FirstScatterType = GetFirstScatterType();
        if (FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
            FirstScatterType == PATHTRACER_SCATTER_REFRACT)
        {
            return PATHTRACER_MASK_CAMERA_TRANSLUCENT;
        }
        if (FirstScatterType == PATHTRACER_SCATTER_SPECULAR)
        {
            return PATHTRACER_MASK_REFLECTION_TRANSLUCENT;
        }
        return PATHTRACER_MASK_INDIRECT_TRANSLUCENT;
    }

	bool UpdateScatterType(uint ScatterType)
	{
		// Keep track of the first non-refractive scatter type, and leave it "locked" beyond this
		// This ensures a particular pixel can only contribute to one lighting component.
		// The reason to not split refractive events is that you typically want to treat refracted paths the same as camera paths.
		// For example, when extracting just the diffuse component of a character wearing glasses, you would want the directly visible diffuse
		// and refracted diffuse to be treated the same.
		if (!HasMadeContributionScatter())
		{
			SetFirstScatterType(ScatterType);
		}
		switch (GetFirstScatterType())
		{
			// In these cases, only keep tracing if the corresponding lighting component is enabled
			case PATHTRACER_SCATTER_DIFFUSE:  return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_DIFFUSE ) != 0;
			case PATHTRACER_SCATTER_SPECULAR: return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_SPECULAR) != 0;
			case PATHTRACER_SCATTER_VOLUME:   return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_VOLUME  ) != 0;
		}
		// In all other cases, keep tracing, we haven't made a final decision on what this path is yet
		return true;
	}

	bool ShouldAccumulateEmissive()
	{
		uint EnabledLightingContributions = HasMadeContributionScatter() ? EnabledIndirectLightingContributions : EnabledDirectLightingContributions;
		return (EnabledLightingContributions & PATHTRACER_CONTRIBUTION_EMISSIVE) != 0;
	}

	// The following functions allow us to scale down the contributions for lighting components before we have scattered
	bool ShouldAccumulateDiffuse()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_DIFFUSE) != 0);
	}

	bool ShouldAccumulateSpecular()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_SPECULAR) != 0);
	}

	bool ShouldAccumulateVolume()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_VOLUME) != 0);
	}

	float2 GetDiffuseSpecularScale(bool bIsVolumeSample)
	{
		float2 Result = 0.0;
		if (bIsVolumeSample)
		{
			if (ShouldAccumulateVolume())
			{
				Result.x = 1.0;
			}
		}
		else
		{
			if (ShouldAccumulateDiffuse())
			{
				Result.x = 1.0;
			}
			if (ShouldAccumulateSpecular())
			{
				Result.y = 1.0;
			}
		}
		return Result;
	}

	void WritePixel(uint2 TextureIndex)
	{
#if PATH_TRACER_USE_ADAPTIVE_SAMPLING
		float4 OldVariance = Iteration > 0 ? VarianceTexture[TextureIndex] : 0;
		const float Blend = 1.0 / (OldVariance.z + 1);
		const float PixelLum = Luminance(Radiance);
		const float2 VarianceValue = float2(PixelLum, PixelLum * PixelLum);
		VarianceTexture[TextureIndex] = float4(lerp(OldVariance.xy, VarianceValue, Blend), OldVariance.z + 1.0, 0.0);
#else
		// non-adaptive sampling, blend factor is constant for all pixels
		const float Blend = BlendFactor;
#endif

		// Avoid reading the old pixel on the first sample on the off-chance there is a NaN/Inf pixel ...
		float4 OldPixel  = Iteration > 0 ? RadianceTexture[TextureIndex] : 0;
		float4 OldAlbedo = Iteration > 0 ? AlbedoTexture[TextureIndex]   : 0;
		float4 OldNormal = Iteration > 0 ? NormalTexture[TextureIndex]   : 0;

		const float4 PixelValue = float4(Radiance, Alpha);

		const float3 Albedo = GetAlbedo();
		const float3 Normal = GetNormal();

		RadianceTexture[TextureIndex] = lerp(OldPixel, PixelValue, Blend);
		AlbedoTexture[TextureIndex].xyz = lerp(OldAlbedo.xyz, Albedo, Blend);
		NormalTexture[TextureIndex].xyz = lerp(OldNormal.xyz,Normal.xyz, Blend);
	}

#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	void WriteDepth(uint2 TextureIndex)
	{
		float OldDepth = Iteration > 0 ? DepthTexture[TextureIndex] : 0;
		DepthTexture[TextureIndex] = max(OldDepth, ConvertToDeviceZ(DepthZ));
	}
#endif


    void AccumulateRadiance(float3 PathRadiance)
    {
        PathRadiance *= BaseExposure;
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
        // camera rays, clamp at a fairly high level for half-float final storage into SceneColor buffer
        float MaxIntensity = MaxHalfFloat;
#else
        // User asked for path contributions to be clamped to reduce fireflies from indirect paths
        // This puts an upper bound on variance within the pixel at the expense of bias
        float MaxIntensity = MaxPathIntensity;
#endif
        // compute factor that will clamp the intensity but preserve the color
        float Adjust = MaxIntensity / max(max(MaxIntensity, PathRadiance.x), max(PathRadiance.y, PathRadiance.z));
        // accumulate into the total
        Radiance += PathRadiance * Adjust;
    }
};

FPackedPathTracingPayload TraceTransparentRay(inout FPathState PathState, int Bounce, inout FVolumeSegment VolumeSegment)
{
	const bool bLastBounce = Bounce == MaxBounces;
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	// backface culling on camera rays only to match raster but still allow refraction and indirect to work properly
	const uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES | GetPathTracingHairRayFlags(PathState.Ray.Origin);
#else
	const uint RayFlags = GetPathTracingHairRayFlags(PathState.Ray.Origin);
#endif
    const uint InstanceInclusionMaskOpaque = PathState.GetInstanceInclusionMaskOpaque();

	const uint MissShaderIndex = 0;
	float3 RISRandSample = PathState.RandSequence.Get3D();
	FRISContext HitSample = InitRISContext(RISRandSample.x);
	FRISContext VolSample = InitRISContext(RISRandSample.y);

	float3 PayloadThroughput = 0;
	FPackedPathTracingPayload Payload;
	Payload.SetMiss();
	if (bLastBounce && !PathState.ShouldAccumulateEmissive())
	{
		PathState.PathThroughput = 0;
		return Payload;
	}

	FPackedPathTracingPayload OpaquePayload = InitPathTracingPayload(PathState.GetFirstScatterType(), ApproximateCaustics * PathState.GetPathRoughness());
	OpaquePayload.SetDBufferA(float4(0, 0, 0, 1));
	OpaquePayload.SetDBufferB(float4(0, 0, 0, 1));
	OpaquePayload.SetDBufferC(float4(0, 0, 0, 1));
	OpaquePayload.SetStochasticSlabRand(RISRandSample.z);
	OpaquePayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT);
#if PATH_TRACER_USE_SER 
	{
		NvHitObject Hit;
		NvTraceRayHitObject(
			TLAS,
			RayFlags,
			InstanceInclusionMaskOpaque,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			PathState.Ray.GetNativeDesc(),
			OpaquePayload,
			Hit);
	#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
		NvReorderThread(Hit);
	#endif
		NvInvokeHitObject(TLAS, Hit, OpaquePayload);
	}
#else
	// Trace the ray
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMaskOpaque,
		RAY_TRACING_SHADER_SLOT_MATERIAL,
		RAY_TRACING_NUM_SHADER_SLOTS,
		MissShaderIndex,
		PathState.Ray.GetNativeDesc(),
		OpaquePayload);
#endif
    if (OpaquePayload.IsDecalReceiver() && PathState.GetPathRoughness() < max(MeshDecalRoughnessCutoff, DecalRoughnessCutoff))
    {
        FDecalShaderPayload DecalPayload = InitDecalShaderPayload(PathState.Ray.Origin + OpaquePayload.HitT * PathState.Ray.Direction);
        // Evaluate mesh decals and decal actors
        EvaluateDecals(
            PathState.Ray.Direction,
            PathState.GetInstanceInclusionMaskOpaque() | PathState.GetInstanceInclusionMaskTranslucent(),
            OpaquePayload.HitT,
            PathState.GetPathRoughness(),
            DecalPayload
        );
        // Handle d-buffer requests (will potentially re-trace the primary ray if needed)
        PATH_TRACER_DECAL_PROCESS_DBUFFER(OpaquePayload, ApproximateCaustics, RISRandSample.z)
        // Now that we know the real decal data, apply it
        ApplyDecal(DecalPayload, OpaquePayload, ApproximateCaustics * PathState.GetPathRoughness());
    }

	if (OpaquePayload.IsHit())
	{
		PathState.Ray.TMax = asfloat(asuint(OpaquePayload.HitT) - 1);
	}

	for (;;)
	{
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(PathState.GetFirstScatterType(), ApproximateCaustics * PathState.GetPathRoughness());
		PackedPayload.SetDBufferA(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferB(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferC(float4(0, 0, 0, 1));
		PackedPayload.SetStochasticSlabRand(RISRandSample.z);
		if (OpaquePayload.IsHit())
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = PathState.Ray.Origin + OpaquePayload.HitT * PathState.Ray.Direction;
			PackedPayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH);
			PackedPayload.SetSceneDepth(dot(TranslatedWorldPos, ViewZ));
		}

        const uint InstanceInclusionMaskTranslucent = PathState.GetInstanceInclusionMaskTranslucent();

#if PATH_TRACER_USE_SER 
		{
			NvHitObject Hit;
			NvTraceRayHitObject(
				TLAS,
				RayFlags,
				InstanceInclusionMaskTranslucent,
				RAY_TRACING_SHADER_SLOT_MATERIAL,
				RAY_TRACING_NUM_SHADER_SLOTS,
				MissShaderIndex,
				PathState.Ray.GetNativeDesc(),
				PackedPayload,
				Hit);
		#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
			NvReorderThread(Hit);
		#endif
			NvInvokeHitObject(TLAS, Hit, PackedPayload);
		}
#else
		// Trace the ray
		TraceRay(
			TLAS,
			RayFlags,
			InstanceInclusionMaskTranslucent,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			PathState.Ray.GetNativeDesc(),
			PackedPayload);
#endif

		if (PackedPayload.IsMiss())
		{
			// If we miss the translucent geometry, the next hit is the opaque one
			PackedPayload = OpaquePayload;
		}

		FVolumeIntersectionList VolumeIsectList = VolumeIntersect(PathState.Ray.Origin, PathState.Ray.Direction, PathState.Ray.TMin, PathState.Ray.TMax, PathState.GetPathRoughness(), VolumeFlags);
		if (VolumeIsectList.HitBlocker() &&
			(PathState.GetFirstScatterType() == PATHTRACER_SCATTER_VOLUME ||  (VolumeFlags & PATH_TRACER_VOLUME_SHOW_PLANET_GROUND) != 0) &&
			(PackedPayload.IsMiss() || VolumeIsectList.BlockerHitT < PackedPayload.HitT))
		{
			// we didn't hit any real geometry, but we did hit the volume's blocker geometry
			// create a virtual hit with it here
			PackedPayload = VolumeGetBlockerHit(PathState.Ray.Origin, PathState.Ray.Direction, VolumeIsectList.BlockerHitT, PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA);
		}

		// Loop over lights to capture their contribution
		// #dxr_todo: if we have lots of lights, having some hierarchical structure would be better ....
		// number of directly visible lights for the first bounce
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		// Run this for camera rays only, after the first bounce, we don't need to include lights in the trace call
		// because NEE handled it for us
		for (uint LightId = 0, NumVisibleLights = SceneVisibleLightCount; LightId < NumVisibleLights; ++LightId)
		{
			FRayDesc LightRay = PathState.Ray;
			LightRay.TMax = PackedPayload.IsMiss() ? PathState.Ray.TMax : PackedPayload.HitT;
			FLightHit LightHit = TraceLight(LightRay, LightId);
			if (LightHit.IsHit())
			{
				// create a virtual surface hit so we only need to worry about volume transmission in one place
				// NOTE: returning only a single light hit here causes lights at infinity to occlude each other
				//       this is most likely what artists would want (imagine a scene with two suns for example)
				//       but is not consistent with how reflections behave and the sorting order will be arbitrary
				FPathTracingPayload LightHitPayload = (FPathTracingPayload)0; // clear all fields
				LightHitPayload.HitT = LightHit.HitT;
				LightHitPayload.Radiance = LightHit.Radiance * SceneLights[LightId].CameraColorScale;
				LightHitPayload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
				// light hits are fully transparent to match how the light loop works (the light surface does not cast a shadow)
				LightHitPayload.BSDFOpacity = 1.0;
				LightHitPayload.TransparencyColor = LightHit.HitT == RAY_DEFAULT_T_MAX ? 1.0 - BackgroundAlpha : 1.0;
				LightHitPayload.SetFrontFace();
				PackedPayload = PackPathTracingPayload(LightHitPayload);
			}
		}
#endif

		if (any(PathState.GetSigmaT() > 0))
		{
			float3 SigmaT = PathState.GetSigmaT();
			float4 ScatterData = PathState.GetSigmaSAndPhase();
			float3 MediumAlbedo = select(SigmaT > 0, ScatterData.xyz / SigmaT, 0.0);
			float3 ColorChannelPdf = PathState.PathThroughput * MediumAlbedo;
			float ScatterT = -1; 
			BRANCH
			if (!bLastBounce && any(ColorChannelPdf > 0))
			{
				// Only sample scatter hits if we have some scattering albedo and this is not the last bounce (which only cares about emission)
				float RandSample = PathState.RandSequence.Get1D();
				ScatterT = SampleSpectralTransmittance(RandSample, SigmaT, ColorChannelPdf);
			}
			float HitT = PackedPayload.IsHit() ? PackedPayload.HitT : PathState.Ray.TMax;
			if (ScatterT < 0)
			{
				// scattering not present, just apply Beer's law through the current medium
				PathState.PathThroughput *= exp(-SigmaT * (HitT - PathState.Ray.TMin));
			}
			else if (PathState.Ray.TMin + ScatterT < HitT)
			{
				float3 PathThroughput = PathState.PathThroughput * EvaluateSpectralTransmittanceHit(ScatterT, SigmaT, ColorChannelPdf).xyz;
				ScatterT += PathState.Ray.TMin;
				float3 Contrib = PathThroughput * ScatterData.xyz;

				// accumulate what the denoiser wants into albedo/normal (as long as the current path is rough enough)
				float3 Albedo = PathState.GetAlbedo();
				AccumulateAlbedo(ScatterData.xyz, PathThroughput, PathState.GetPathRoughness(), Albedo);
				PathState.SetAlbedoNormal(Albedo, PathState.GetNormal());

				float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
				if (HitSample.Accept(SelectionWeight))
				{
					FVolumeShadedResult Result = (FVolumeShadedResult) 0;
					Result.SigmaT = SigmaT;
					Result.SigmaSHG = ScatterData.xyz;
					Result.PhaseG = ScatterData.w;
					// stash hit
					Payload = PackPathTracingPayload(CreateMediumHitPayload(ScatterT, PathState.Ray.Origin + ScatterT * PathState.Ray.Direction, Result));
					PayloadThroughput = Contrib / SelectionWeight;
				}
				break; // no more hits left
			}
			else
			{
				// no hit in scattering medium, continue to next surface hit
				PathState.PathThroughput *= EvaluateSpectralTransmittanceMiss(HitT - PathState.Ray.TMin, SigmaT, ColorChannelPdf).xyz;
			}

			// Pretend like we didn't hit any volume to exlude them from the internal medium we are going through
			// It is fairly unlikely (and probably undesireable) to want atmo/cloud/fog to be rendererd inside glass/water.
			// We may eventually want to support heterogeneous volumes in these cases, but this will require unifying the ray marching of the ambient medium with the current one
			VolumeIsectList.VolumeMask = 0;
		}


		// Volume Transmittance + Scatter Segment/Point selection
		// This loop splits the potential volumetric segments into sub-intervals that do not overlap.
		while (VolumeIsectList.HitVolume())
		{
			// extract the nearest interval from the list of segments
			FVolumeIntersectionInterval Interval = VolumeIsectList.GetCurrentInterval();

			if (PackedPayload.IsHit())
			{
				if (PackedPayload.HitT <= Interval.VolumeTMin)
				{
					// surface hit is in-front of the current interval, we are done
					break;
				}
				// clip current volume segment to the next surface hit
				Interval.VolumeTMax = min(PackedPayload.HitT, Interval.VolumeTMax);
			}

			// if this isn't our last bounce and we are including volumetric contributions, potentially keep this segment for later so we can integrate local lighting
			if (!bLastBounce && PathState.ShouldAccumulateVolume())
			{
				// TODO: can we improve the selection probability for each ray segment somehow?
				// NOTE: this only matters if we are tracing through several transparent hits or if there are multiple overlapping volumes
				float3 Contrib = PathState.PathThroughput;
				float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
				if (VolSample.Accept(SelectionWeight))
				{
					// store this volume segment for later
					VolumeSegment.Throughput = PathState.PathThroughput / SelectionWeight;
					VolumeSegment.Interval = Interval;
				}
			}


#if PATH_TRACER_USE_CLOUD_SHADER
			if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_CLOUDS)
			{
				// let callable shader handle cloud sampling
				FPackedPathTracingPayload CloudPayload = (FPackedPathTracingPayload) 0;
				const bool bNeedSample = !bLastBounce;
				CloudPayload.SetVolumetricCallableShaderInput(
					PathState.Ray.Origin,
					PathState.Ray.Direction,
					Interval.VolumeTMin,
					Interval.VolumeTMax,
					PathState.RandSequence,
					Interval.VolumeMask |
					(VolumeFlags & ~PATH_TRACER_VOLUME_ENABLE_MASK) |
					min(Bounce << PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_SHIFT, PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_MASK) |
					(bNeedSample  ? PATH_TRACER_VOLUME_CALLABLE_FLAGS_GET_SAMPLE    : 0),
					PathState.PathThroughput,
					Interval.CloudDensity
				);
				CloudPayload.SetVolumetricCallableShaderRISContext(HitSample);
				CallShader(CloudCallableShaderId, CloudPayload);
				PathState.RandSequence = CloudPayload.GetVolumetricCallableShaderOutputRandSeq();
				PathState.PathThroughput = CloudPayload.GetVolumetricCallableShaderOutputThroughput();
				PathState.SetAlbedoNormal(
					PathState.GetAlbedo() + CloudPayload.GetVolumetricCallableShaderOutputAlbedo() * DenoiserAOVWeight(PathState.GetPathRoughness()),
					PathState.GetNormal()
				);
				HitSample = CloudPayload.GetVolumetricCallableShaderRISContext();
				PathState.AccumulateRadiance(CloudPayload.GetVolumetricCallableShaderOutputRadiance());
			#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
				PathState.Alpha += CloudPayload.GetVolumetricCallableShaderOutputAlpha();
			#endif
				if (CloudPayload.HitT > 0)
				{
					// If we registered a hit, transfer it to the currently sampled payload
					PayloadThroughput = CloudPayload.GetVolumetricCallableShaderOutputPayloadThroughput();
					FPathTracingPayload Result = (FPathTracingPayload)0; // clear all fields
					Result.HitT = CloudPayload.HitT;
					Result.TranslatedWorldPos = PathState.Ray.Origin + PathState.Ray.Direction * CloudPayload.HitT;
					Result.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_MEDIUM;
					Result.BSDFOpacity = 1.0;
					Result.PrimitiveLightingChannelMask = 7;
					Result.SetFrontFace();
					Result.SetBaseColor(CloudPayload.GetVolumetricCallableShaderOutputRayleighWeight());
					float4 HGData = CloudPayload.GetVolumetricCallableShaderOutputHG();
					Result.SetHG(HGData.xyz, HGData.w);
					Result.SetDualHG(
						CloudPayload.GetVolumetricCallableShaderOutputDualHGWeight(),
						CloudPayload.GetVolumetricCallableShaderOutputDualHGPhaseData()
					);
					Result.SetCloudFactor(CloudPayload.GetVolumetricCallableShaderOutputCloudDensityFactor());
					Payload = PackPathTracingPayload(Result);
				}
			}
			else
#endif // PATH_TRACER_USE_CLOUD_SHADER
			{
				// In order to use analytic transmittance, the user must have requested it _and_ we must be on an interval with no clouds or heterogeneous volumes
				const bool bUseAnalyticTransmittance =
					(VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE) != 0 &&
					(Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES) == 0;
				// are there any holdouts on the current segment?
				const bool bHaveHoldouts =
					(Interval.VolumeMask << PATH_TRACER_MAX_VOLUMES) & VolumeFlags & PATH_TRACER_VOLUME_HOLDOUT_MASK != 0;
				// reasons to enter the ray marching loop:
				//  - we can't use analytic transmittance (need ray marching for throughput or emission)
				//  - we are a camera ray and some volumes are flagged as holdouts
				//  - this is not the last bounce, so we need to select a point for scattering
				if (!bUseAnalyticTransmittance || (PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA && bHaveHoldouts) || !bLastBounce)
				{
					// remember the start of our interval before ray marching messes with it
					float StartVolumeTMin = Interval.VolumeTMin;
					// Ray marching loop
					float3 PathThroughput = PathState.PathThroughput;

					float3 VolumeRadiance = 0;
					// Limit number of steps to prevent timeouts
					// TODO: This biases the result! Is there a better way to limit the number of steps?
					for (int Step = 0; Step < MaxRaymarchSteps; Step++)
					{
						FVolumeTrackingResult TrackingResult = VolumeSampleDistance(PathThroughput, PathState.Ray.Origin, PathState.Ray.Direction, Interval, PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA, PathState.RandSequence);
						if (TrackingResult.Distance < 0)
						{
							break;
						}

						if (TrackingResult.bIsCollision)
						{
							Interval.VolumeTMin = TrackingResult.Distance;
							PathThroughput = TrackingResult.Throughput;

							// find out how much volume exists at the current point
							float3 Ro = PathState.Ray.Origin;
							float3 Rd = PathState.Ray.Direction;
							float3 TranslatedWorldPos = Ro + Interval.VolumeTMin * Rd;
							FVolumeShadedResult Result = VolumeGetDensity(TranslatedWorldPos, TrackingResult.HeterogeneousVolumesIndex, Interval.VolumeMask, PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA);

							// clamp to make sure we never exceed the majorant (should not be the case, but need to avoid any possible numerical issues)
							float3 SigmaT = min(Result.SigmaT, TrackingResult.SigmaBar);
							float3 SigmaN = TrackingResult.SigmaBar - SigmaT;

						#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
							float3 SigmaH = min(Result.SigmaH, SigmaT);
							PathState.Alpha += Luminance(PathThroughput * (SigmaT - SigmaH));
						#endif

							// if this is not the last bounce, consider the possibility of scattering to give us medium hits
							if (!bLastBounce)
							{
								float3 SigmaS = min(Result.SigmaSRayleigh + Result.SigmaSHG + Result.SigmaSDualHG, SigmaT);
								// accumulate a signal for the denoiser
								float3 Albedo = PathState.GetAlbedo();
								AccumulateAlbedo(SigmaS, PathThroughput, PathState.GetPathRoughness(), Albedo);
								PathState.SetAlbedoNormal(Albedo, PathState.GetNormal());
								float3 Contrib = PathThroughput * SigmaS;
								float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
								if (HitSample.Accept(SelectionWeight))
								{
									// stash this hit for next time
									Payload = PackPathTracingPayload(CreateMediumHitPayload(Interval.VolumeTMin, TranslatedWorldPos, Result));
									PayloadThroughput = Contrib / SelectionWeight;
								}
							}

							VolumeRadiance += Result.Emission * PathThroughput;
							// keep tracing through the volume
							PathThroughput *= SigmaN;
							PathThroughput *= LowThroughputClampFactor(PathThroughput);
							if (!any(PathThroughput > 0))
							{
								break;
							}
						}
						else
						{
							// update the path throughput, knowing that we escaped the medium
							PathThroughput = TrackingResult.Throughput;
							// exit the ray marching loop
							break;
						}
					}
					// accumulate all at once so clamping applies to entire volume contribution
					PathState.AccumulateRadiance(VolumeRadiance);
					// restore interval start now that we are finished with ray marching
					Interval.VolumeTMin = StartVolumeTMin;

					if (bUseAnalyticTransmittance)
					{
						// analytically handle the transmittance to the next surface (which will be higher quality than the transmittance implicitly computed above and prevent noise with emissive surfaces in volumes)
						PathState.PathThroughput = VolumeGetTransmittance(PathState.PathThroughput, PathState.Ray.Origin, PathState.Ray.Direction, Interval, PathState.RandSequence);
					}
					else
					{
						// record the change in transmittance since we already computed it
						PathState.PathThroughput = PathThroughput;
					}
				}
			}

			// Update our current volume intersection list to reflect the fact that we have made it to the end of this interval
			// This will either clip the finished portions of active segments, or remove the segments we have fully marched through
			VolumeIsectList = VolumeIsectList.Update(Interval.VolumeTMax);
		}
		// proceed to handling the surface hit (if any)
#if NEED_TMIN_WORKAROUND // extra safety - discard hit if not conforming
		if (PackedPayload.HitT <= PathState.Ray.TMin)
		{
			PackedPayload.HitT = -1.0;
		}
#endif
		if (PackedPayload.IsMiss())
		{
			// Ray didn't hit any real geometry
		#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
			PathState.Alpha += Luminance(PathState.PathThroughput) * BackgroundAlpha;
		#endif
			break;
		}

		#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		if (!PackedPayload.IsHoldout())
		{
			PathState.Alpha += Luminance(PathState.PathThroughput * (1.0 - PackedPayload.UnpackTransparencyColor()));
		}
		#endif

		// add in surface emission
		if (PathState.ShouldAccumulateEmissive())
		{
			PathState.AccumulateRadiance(PathState.PathThroughput * PackedPayload.UnpackRadiance());
		}

		if (PackedPayload.HitT == RAY_DEFAULT_T_MAX)
		{
			// if our hit was against an infinite light, exit now
			break;
		}

	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		if (PackedPayload.ShouldOutputDepth() && PathState.DepthZ == 0.0)
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = PathState.Ray.Origin + PackedPayload.HitT * PathState.Ray.Direction;
			PathState.DepthZ = dot(TranslatedWorldPos, ViewZ);
		}
	#endif

		if (!bLastBounce)
		{
			FPathTracingPayload HitPayload = UnpackPathTracingPayload(PackedPayload, PathState.Ray);
			AdjustPayloadAfterUnpack(HitPayload, ApplyDiffuseSpecularOverrides);

			float3 Contrib = PathState.PathThroughput * EstimateMaterialAlbedo(HitPayload);

			// accumulate what the denoiser wants into albedo/normal (as long as the current path is rough enough)
			float3 Albedo = PathState.GetAlbedo();
			float3 Normal = PathState.GetNormal();
			AccumulateAlbedoNormal(HitPayload, PathState.PathThroughput, PathState.GetPathRoughness(), Albedo, Normal);
			PathState.SetAlbedoNormal(Albedo, Normal);

			float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
			if (HitSample.Accept(SelectionWeight))
			{
				// stash this hit for next time
				Payload = PackedPayload;
				PayloadThroughput = PathState.PathThroughput / SelectionWeight;
			}
		}

		// account for local transparency change
		PathState.PathThroughput *= PackedPayload.UnpackTransparencyColor();

		// prepare next step around the loop
		// retrace the exact same ray with TMin one ulp past the hit we just found
		PathState.Ray.TMin = ComputeNewTMin(PathState.Ray.Origin, PathState.Ray.Direction, PackedPayload.HitT);
		if (!(PathState.Ray.TMin < PathState.Ray.TMax))
		{
			break;
		}

		if (all(PathState.PathThroughput == 0))
		{
			break;
		}
	}

	// normalization will be 0 if we didn't pick any volume segment
	VolumeSegment.Throughput *= VolSample.GetNormalization();

	// if we stored a valid hit in the payload, reset the path throughput to this point
	// normalization will be 0 if we didn't pick any hit
	PathState.PathThroughput = PayloadThroughput * HitSample.GetNormalization();

	return Payload;
}

float3 TraceTransparentVisibilityRay(FRayDesc Ray, float PathRoughness, float PreviousPathRoughness, uint MissShaderIndex, bool bCastShadows, bool bCloudCastShadows, float CloudFactor, inout FRandomSequence RandSequence)
{
	if (!bCastShadows && MissShaderIndex == 0)
	{
		// no work to do
		return 1.0;
	}

    const uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER | GetPathTracingHairRayFlags(Ray.Origin);
	const uint InstanceInclusionMask = PATHTRACER_MASK_SHADOW;
	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_SHADOW;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;

	FPackedPathTracingPayload PackedPayload = InitPathTracingVisibilityPayload(PathRoughness);

	if (!bCastShadows)
	{
		// ray should not cast shadows, make it degenerate so we can still run the miss shader
		Ray.TMin = POSITIVE_INFINITY;
		Ray.TMax = POSITIVE_INFINITY;
	}
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);

	if (PackedPayload.IsHit())
	{
		// We didn't run the miss shader, therefore we must have hit something opaque (or reached full opacity)
		return 0.0;
	}

	float3 Throughput = PackedPayload.GetRayThroughput();
	if (bCastShadows && any(Throughput > 0))
	{
		// NOTE: use the path roughness before scattering off the current vertex (so we can have clouds casting shadows onto diffuse surfaces)
		uint VolumeMask = VolumeFlags;
		if (!bCloudCastShadows || CloudFactor <= 0)
		{
			// if skipping clouds on shadows, clear the contribution here
			VolumeMask &= ~PATH_TRACER_VOLUME_ENABLE_CLOUDS;
		}

		FVolumeIntersectionList VolumeIsectList = VolumeIntersect(Ray.Origin, Ray.Direction, Ray.TMin, Ray.TMax, PreviousPathRoughness, VolumeMask);
		if (VolumeIsectList.HitBlocker())
		{
			// blockers are opaque -- done!
			return 0.0;
		}
		if (VolumeIsectList.HitVolume())
		{
			Throughput = VolumeGetTransmittance(Throughput, Ray.Origin, Ray.Direction, VolumeIsectList, RandSequence, CloudFactor);
		}
		// add in the medium extinction
		Throughput *= exp(-max(PackedPayload.GetTau(), 0.0));
	}
	return Throughput;
}

struct FProbeResult
{
	float HitT;
	float3 WorldNormal;
	float3 WorldSmoothNormal;
	float3 WorldGeoNormal;
	int FrontFace;

	bool IsMiss() { return HitT <= 0; }
};

FProbeResult TraceSSSProbeRay(FRayDesc Ray, inout int InterfaceCounter)
{
	// Use the technique mentioned in "A Hero Beneath the Surface" (Section 6.3.2)
	// https://jo.dreggn.org/home/2021_spectral_imaging.pdf
	// This allows multiple overlapping meshes to be treated as a single "volume" by discarding internal hits until we have
	// crossed the appropriate number of interfaces
#define SSS_USE_INTERFACE_COUNTING 1

#if SSS_USE_INTERFACE_COUNTING
	for (;;)
#endif
	{
		// Trace a short ray to see if we escaped the surface
		// NOTE: we leave these rays tagged as "camera" because we are interested in getting the same shading normal the camera saw (on the off chance the material does some ray-simplfiication)
		//      Since the SSS code mostly runs on camera rays, this is a reasonable approximation
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(PATHTRACER_SCATTER_CAMERA, 1.0);
		// TODO: use smaller payload + flag to skip shading work? (only want smooth+geo normals)
		//       however supporting blockers would require some shading ...
		//       
		const uint RayFlags = RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES; // ignore procedural hair
		const uint InstanceInclusionMask = PATHTRACER_MASK_SSS_RANDOM_WALK;
		const uint MissShaderIndex = 0;
		TraceRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			Ray.GetNativeDesc(),
			PackedPayload);
#if NEED_TMIN_WORKAROUND // extra safety - discard hit if not conforming
		if (PackedPayload.HitT <= Ray.TMin)
		{
			PackedPayload.HitT = -1.0;
		}
#endif
		if (PackedPayload.IsMiss())
		{
			// we did not hit anything
			return (FProbeResult)0;
		}

#if SSS_USE_INTERFACE_COUNTING
		// update counter
		InterfaceCounter += PackedPayload.IsFrontFace() ? +1 : -1;
		if (InterfaceCounter != 0)
		{
			// we have not yet crossed the right number of interfaces, so ignore the current hit and try the segment again
			Ray.TMin = ComputeNewTMin(Ray.Origin, Ray.Direction, PackedPayload.HitT);
			continue;
		}
#endif

		FPathTracingPayload Payload = UnpackPathTracingPayload(PackedPayload, Ray);
		FProbeResult Result;
		Result.HitT = PackedPayload.HitT;
		Result.WorldNormal = Payload.WorldNormal;
		Result.WorldSmoothNormal = Payload.WorldSmoothNormal;
		Result.WorldGeoNormal = Payload.WorldGeoNormal;
		Result.FrontFace = Payload.IsFrontFace();
		return Result;
	}
}

float3 ComputeDwivediScale(float3 Albedo)
{
	// "Zero-Variance Theory for Efficient Subsurface Scattering"
	// Eugene d'Eon and Jaroslav Krivanek (SIGGRAPH 2020)
	// http://www.eugenedeon.com/project/zerovar2020/
	// Eq. 67
	const float3 ClampedAlbedo = clamp(Albedo, 0.001, 0.999); // avoid singularities at the extremeties
	return rsqrt(1.0 - pow(ClampedAlbedo, 2.44294 - 0.0215813 * ClampedAlbedo + 0.578637 / ClampedAlbedo));
}

float SampleGuidedSpectralTransmittance(float RandValue, float SlabCosine, float3 DwivediScale, float GuidingFactor, float3 Sigma, float3 ProbT, float3 ColorChannelPdf)
{
	float3 ColorChannelCdf = float3(
		ColorChannelPdf.x,
		ColorChannelPdf.x + ColorChannelPdf.y,
		ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	if (ColorChannelCdf.z > 0)
	{
		
		// choose guided vs non-guided
		if (RandValue < GuidingFactor)
		{
			// dwivedi walk, stretch sigma
			RandValue = RescaleRandomNumber(RandValue, 0.0, GuidingFactor);
		}
		else
		{
			// classic walk, use unmodified sigma
			SlabCosine = 0;
			RandValue = RescaleRandomNumber(RandValue, GuidingFactor, 1.0);

		}
		const float q = RandValue * ColorChannelCdf.z;
		if (q < ColorChannelCdf.x)
		{
			const float RescaleRand1 = RescaleRandomNumber(q, 0.0, ColorChannelCdf.x);
			const float RescaleRand2 = RescaleRand1 < ProbT.x ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.x) : RescaleRandomNumber(RescaleRand1, ProbT.x, 1.0);
			SlabCosine *= RescaleRand1 < ProbT.x ? -1.0 : 1.0;
			const float StretchedSigma = Sigma.x * (1 - SlabCosine / DwivediScale.x);
			return -log(1 - RescaleRand2) / StretchedSigma;
		}
		else if (q < ColorChannelCdf.y)
		{
			const float RescaleRand1 = RescaleRandomNumber(q, ColorChannelCdf.x, ColorChannelCdf.y);
			const float RescaleRand2 = RescaleRand1 < ProbT.y ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.y) : RescaleRandomNumber(RescaleRand1, ProbT.y, 1.0);
			SlabCosine *= RescaleRand1 < ProbT.y ? -1.0 : 1.0;
			const float StretchedSigma = Sigma.y * (1 - SlabCosine / DwivediScale.y);
			return -log(1 - RescaleRand2) / StretchedSigma;

		}
		else
		{
			const float RescaleRand1 = RescaleRandomNumber(q, ColorChannelCdf.y, ColorChannelCdf.z);
			const float RescaleRand2 = RescaleRand1 < ProbT.z ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.z) : RescaleRandomNumber(RescaleRand1, ProbT.z, 1.0);
			SlabCosine *= RescaleRand1 < ProbT.z ? -1.0 : 1.0;
			const float StretchedSigma = Sigma.z * (1 - SlabCosine / DwivediScale.z);
			return -log(1 - RescaleRand2) / StretchedSigma;
		}
	}
	// all channels have 0 probability
	return -1.0;
}

float4 EvaluateGuidedSpectralTransmittanceHit(float SampledT, float SlabCosine, float3 DwivediScale, float GuidingFactor, float3 Sigma, float3 ProbT, float3 ColorChannelPdf)
{
	// normalize the pdf (to match code above)
	ColorChannelPdf *= rcp(ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	float3 Transmittance = exp(-SampledT * Sigma);
	// probability of reaching the sampled point with classic sampling
	float3 TransmittancePdf = Sigma * Transmittance;
	// probability of reaching the sampled point with guiding
	float3 GuidedSigmaR = (1 - SlabCosine / DwivediScale) * Sigma;
	float3 GuidedSigmaT = (1 + SlabCosine / DwivediScale) * Sigma;
	float3 GuidedPdfR = GuidedSigmaR * exp(-SampledT * GuidedSigmaR);
	float3 GuidedPdfT = GuidedSigmaT * exp(-SampledT * GuidedSigmaT);
	float3 GuidedPdf = lerp(GuidedPdfR, GuidedPdfT, ProbT);
	float MisPdf = dot(ColorChannelPdf, lerp(TransmittancePdf, GuidedPdf, GuidingFactor));
	return MisPdf > 0 ? float4(Transmittance / MisPdf, MisPdf) : 0.0;
}

float4 EvaluateGuidedSpectralTransmittanceMiss(float MaxT, float SlabCosine, float3 DwivediScale, float GuidingFactor, float3 Sigma, float3 ProbT, float3 ColorChannelPdf)
{
	// normalize the pdf (to match code above)
	ColorChannelPdf *= rcp(ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	float3 Transmittance = exp(-MaxT * Sigma);
	float3 TransmittancePdf = Transmittance; // probability of going past MaxT with classic sampling

	// probability of going past MaxT (integral of the pdf from MaxT to infinity) with guiding
	float3 GuidedSigmaR = (1 - SlabCosine / DwivediScale) * Sigma;
	float3 GuidedSigmaT = (1 + SlabCosine / DwivediScale) * Sigma;
	float3 GuidedPdfR = exp(-MaxT * GuidedSigmaR);
	float3 GuidedPdfT = exp(-MaxT * GuidedSigmaT);
	float3 GuidedPdf = lerp(GuidedPdfR, GuidedPdfT, ProbT);
	
	float MisPdf = dot(ColorChannelPdf, lerp(TransmittancePdf, GuidedPdf, GuidingFactor));
	return MisPdf > 0 ? float4(Transmittance / MisPdf, MisPdf) : 0.0;
}

// returns: xyz: world space direction, w: throughput scale
float4 SampleDwivediPhaseFunction(float3 ColorChannelPdf, float3 DwivediScale, float GuidingFraction, float3 ProbT, float3 DwivediSlabNormal, float3 RayDirection, float G, float2 RandSample)
{
	float4 Result = 0;
	float3 ColorChannelCdf = float3(
		ColorChannelPdf.x,
		ColorChannelPdf.x + ColorChannelPdf.y,
		ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	if (ColorChannelCdf.z > 0)
	{
		const float3 PhaseLog = log((DwivediScale + 1.0) / (DwivediScale - 1.0));
		const float OneMinusEpsilon = 0.99999994; // 32-bit float just before 1.0
		if (RandSample.x < GuidingFraction)
		{
			// sample the dwivedi guiding pdf
			RandSample.x = RescaleRandomNumber(RandSample.x, 0.0, GuidingFraction);
			const float q = RandSample.x * ColorChannelCdf.z;
			float CosineZ = 0;
			float Sign = 1;
			if (q < ColorChannelCdf.x)
			{
				const float RescaleRand1 = RescaleRandomNumber(q, 0.0, ColorChannelCdf.x);
				const float RescaleRand2 = RescaleRand1 < ProbT.x ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.x) : RescaleRandomNumber(RescaleRand1, ProbT.x, 1.0);
				CosineZ = (DwivediScale.x - (DwivediScale.x + 1) * exp(-RescaleRand2 * PhaseLog.x));
				Sign = RescaleRand1 < ProbT.x ? -1.0 : +1.0;
			}
			else if (q < ColorChannelCdf.y)
			{
				const float RescaleRand1 = RescaleRandomNumber(q, ColorChannelCdf.x, ColorChannelCdf.y);
				const float RescaleRand2 = RescaleRand1 < ProbT.y ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.y) : RescaleRandomNumber(RescaleRand1, ProbT.y, 1.0);
				CosineZ = (DwivediScale.y - (DwivediScale.y + 1) * exp(-RescaleRand2 * PhaseLog.y));
				Sign = RescaleRand1 < ProbT.y ? -1.0 : +1.0;
			}
			else
			{
				const float RescaleRand1 = RescaleRandomNumber(q, ColorChannelCdf.y, ColorChannelCdf.z);
				const float RescaleRand2 = RescaleRand1 < ProbT.z ? RescaleRandomNumber(RescaleRand1, 0.0, ProbT.z) : RescaleRandomNumber(RescaleRand1, ProbT.z, 1.0);
				CosineZ = (DwivediScale.z - (DwivediScale.z + 1) * exp(-RescaleRand2 * PhaseLog.z));
				Sign = RescaleRand1 < ProbT.z ? -1.0 : +1.0;

			}
			float3 PhasePdfR = rcp((DwivediScale - CosineZ) * PhaseLog * (2 * PI));
			float3 PhasePdfT = rcp((DwivediScale + CosineZ) * PhaseLog * (2 * PI));
			float3 PhasePdf = lerp(PhasePdfR, PhasePdfT, ProbT);
			ColorChannelPdf *= rcp(ColorChannelCdf.z);
			float MisPdf = dot(ColorChannelPdf, PhasePdf);


			float SineZ = sqrt(saturate(1 - CosineZ * CosineZ));
			float Phi = (2 * PI) * RandSample.y;
			// produce output direction in slab tangent frame
			Result.xyz = normalize(TangentToWorld(float3(SineZ * cos(Phi), SineZ * sin(Phi), Sign * CosineZ), DwivediSlabNormal));
			// final throughput is the phase function divided by pdf
			float PhaseCosine = -dot(RayDirection, Result.xyz);
			float PhaseEval = HenyeyGreensteinPhase(G, PhaseCosine);
			Result.w = PhaseEval / lerp(PhaseEval, MisPdf, GuidingFraction);
		}
		else
		{
			// sample the classic HG lobe directly (but include probability of sampling the guided lobe)
			RandSample.x = RescaleRandomNumber(RandSample.x, GuidingFraction, 1.0);
			float4 DirectionAndPhase = ImportanceSampleHenyeyGreensteinPhase(RandSample, G);
			Result.xyz = normalize(TangentToWorld(DirectionAndPhase.xyz, RayDirection));

			float CosineZ = dot(Result.xyz, DwivediSlabNormal);

			float3 GuidedPhasePdfR = rcp((DwivediScale - CosineZ) * PhaseLog * (2 * PI));
			float3 GuidedPhasePdfT = rcp((DwivediScale + CosineZ) * PhaseLog * (2 * PI));
			float3 GuidedPhasePdf = lerp(GuidedPhasePdfR, GuidedPhasePdfT, ProbT);
			ColorChannelPdf *= rcp(ColorChannelCdf.z);
			float MisPdf = dot(ColorChannelPdf, GuidedPhasePdf);

			// final throughput is the phase function divided by pdf
			float PhaseEval = DirectionAndPhase.w;
			Result.w = PhaseEval / lerp(PhaseEval, MisPdf, GuidingFraction);
		}
	}
	return Result;
}

bool ProcessSubsurfaceRandomWalk(inout FPathTracingPayload Payload, inout float3 PathThroughput, inout FRandomSequence RandSequence, float3 RayDirection, bool SimplifySSS)
{
	if (!Payload.IsSubsurfaceMaterial())
	{
		// material doesn't have SSS
		// just return so we can process the rest of the payload's shading
		return true;
	}

	FSSSRandomWalkInfo SSS = GetMaterialSSSInfo(Payload, -RayDirection);


	if (SimplifySSS || all(SSS.Radius == 0) || all(SSS.Color == 0) || MaxSSSBounces == 0)
	{
		// if we are running in a context that won't see the SSS directly -- just turn it into diffuse and skip the random walk
		// we can also skip random walk if the radius or color is 0
		// or if the user decided not to perform any SSS bounces
		RemoveMaterialSSS(Payload);
		return true;
	}
	
	// decide randomly to evaluate the SSS part of the material, or keep shading the current part
	float3 RandSample = RandSequence.Get3D();
	if (RandSample.x < SSS.Prob)
	{
		// do random walk (and adjust throughput to remove energy reflected away by the specular lobe)
		PathThroughput *= SSS.Weight / SSS.Prob;
		RandSample.x /= SSS.Prob;
	}
	else
	{
		// skip random walk and shade current point instead
		PathThroughput *= 1 / (1 - SSS.Prob);
		return true;
	}

	// create ray to enter the surface (using a diffuse scattering event)
	FRayDesc Ray;
	Ray.Origin = Payload.TranslatedWorldPos;
	Ray.Direction = TangentToWorld(-CosineSampleHemisphere(RandSample.xy).xyz, Payload.WorldNormal);
	Ray.TMin = 0;
	ApplyRayBias(Ray.Origin, Payload.HitT, -Payload.WorldGeoNormal);

	// Support SSS for mfp smaller than 1mm.
	SSS.Radius = max(SSS.Radius, 0.0009); // 0.009mm minimum to make sure random walk can move forward and to keep SigmaT finite


	int InterfaceCounter = Payload.IsFrontFace() ? +1 : -1;

	// Use the technique detailed in the Hyperion paper (Section 4.4.2)
	// https://media.disneyanimation.com/uploads/production/publication_asset/177/asset/a.pdf
	// This avoids overly bright thin regions by applying a bit of internal reflection when the boundary is hit
#define SSS_USE_TIR 1


#if SSS_USE_TIR
	float3 Albedo = 1 - exp(SSS.Color * (-11.43 + SSS.Color * (15.38 - 13.91 * SSS.Color)));
#else
	// Van de-Hulst inverse mapping
	// https://blog.selfshadow.com/publications/s2017-shading-course/imageworks/s2017_pbs_imageworks_slides_v2.pdf (Slide 44)
	// http://www.eugenedeon.com/project/a-hitchhikers-guide-to-multiple-scattering/ (Section 7.5.3 of v0.1.3)
	float3 Albedo = 1 - Pow2(4.09712 + 4.20863 * SSS.Color - sqrt(9.59217 + SSS.Color * (41.6808 + 17.7126 * SSS.Color)));
	SSS.Radius *= 2.0; // roughly match parameterization above
#endif

	// Subsurface guiding is implemented following the Dwivedi random walk technique described here:
	// https://cgg.mff.cuni.cz/~jaroslav/papers/2014-zerovar/
	// http://www.eugenedeon.com/project/zerovar2020/
	// A thin-slab approximation is used to improve the guiding in thin regions as well as described in the video presentation (slides 37-39).
#define SSS_USE_DWIVEDI 1
#define SSS_USE_DWIVEDI_USE_THIN_SLABS 1 // Probe the geometry to have an estimate of thickness - and use this to guide toward front or backside, depending on which is closer

	// Revisiting Physically Based Shading at Imageworks.
	// https://blog.selfshadow.com/publications/s2017-shading-course/imageworks/s2017_pbs_imageworks_slides_v2.pdf
	float G = SSS.G;
	Albedo = Albedo / (1 - G * (1 - Albedo));

#if SSS_USE_DWIVEDI
	// Setup an oriented slab to approximate the surface. This is used to guide paths back towards the surface so they can escape
	const float3 DwivediScale = ComputeDwivediScale(Albedo);
	float3 DwivediSlabNormal = Payload.WorldSmoothNormal;
	float3 DwivediSlabOrigin = Payload.TranslatedWorldPos;
	// Guiding only works for isotropic phase functions - limit the guiding to a narrow range of phase functions around G=0.0
	// This heuristic was compared to the one from:
	// https://jo.dreggn.org/home/2016_dwivedi.pdf (see Equation 15)
	// but the latter does not appear sufficient for G > 0.25 or so. This is potentially explained by the lack of Closest point or incident illumination guiding.
	const float GuidedRatio = SSSGuidingRatio * (1.0 - pow(saturate(abs(G * 4)), 0.0625));

#if SSS_USE_DWIVEDI_USE_THIN_SLABS
	bool bDoSlabSearch = GuidedRatio > 0;
	float SlabThickness = -1.0; // negative means we didn't find a valid thickness
#endif

#endif // SSS_USE_DWIVEDI


	const int MAX_SSS_BOUNCES = MaxSSSBounces;
	const float3 SigmaT = rcp(SSS.Radius);
	const float3 SigmaS = Albedo * SigmaT;
	for (int i = 0; i < MAX_SSS_BOUNCES; i++)
	{
		float3 ColorChannelPdf = PathThroughput * Albedo;
#if SSS_USE_DWIVEDI
		float SlabCosine = dot(Ray.Direction, DwivediSlabNormal);
#if SSS_USE_DWIVEDI_USE_THIN_SLABS
		if (bDoSlabSearch)
		{
			FRayDesc ProbeRay;
			ProbeRay.Origin = Ray.Origin;
			ProbeRay.Direction = -DwivediSlabNormal;
			ProbeRay.TMin = 0.0;
			ProbeRay.TMax = 10 * max3(SSS.Radius.x, SSS.Radius.y, SSS.Radius.z);
			int ProbeInterfaceCounter = InterfaceCounter;
			FProbeResult Result = TraceSSSProbeRay(ProbeRay, ProbeInterfaceCounter);
			if (Result.IsMiss())
			{
				// didn't find a hit, register missing slab
				SlabThickness = -1.0;
			}
			else
			{
				// got a valid hit -- use it as our thickness
				SlabThickness = Result.HitT;
			}
			bDoSlabSearch = false;
		}

		// Instead of only guiding towards the slab front (reflection), also guide toward the slack back (tranmission) when the surface is thin
		// The heuristic to choose between guiding front or back is determined by the following probability, given in [2] on slide 37 of the video
		// Note that the depths in the video presentation are optical depths, so have to be multiplied by SigmaT
		float SlabZ = clamp(dot(DwivediSlabOrigin - Ray.Origin, DwivediSlabNormal), 0.0, SlabThickness);
		float3 ProbT = SlabThickness > 0.0 ? rcp(1 + exp(SigmaT * (SlabThickness - 2 * SlabZ) / DwivediScale)) : 0.0;
#else
		float3 ProbT = 0.0;
#endif
		Ray.TMax = SampleGuidedSpectralTransmittance(RandSample.z, SlabCosine, DwivediScale, GuidedRatio, SigmaT, ProbT, ColorChannelPdf);
#else // SSS_USE_DWIVEDI
		Ray.TMax = SampleSpectralTransmittance(RandSample.z, SigmaT, ColorChannelPdf);
#endif
		if (Ray.TMax < 0.0)
		{
			// no more energy left in the path
			break;
		}

		FProbeResult ProbeResult = TraceSSSProbeRay(Ray, InterfaceCounter);

		RandSample = RandSequence.Get3D();
		if (ProbeResult.IsMiss())
		{
			// we didn't hit anything, so scatter according to the scattering distribution in the volume and keep tracing
			Ray.Origin += Ray.TMax * Ray.Direction;

#if SSS_USE_DWIVEDI
			// account for transmittance and scattering up to this point
			PathThroughput *= SigmaS * EvaluateGuidedSpectralTransmittanceHit(Ray.TMax, SlabCosine, DwivediScale, GuidedRatio, SigmaT, ProbT, ColorChannelPdf).xyz;

#if SSS_USE_DWIVEDI_USE_THIN_SLABS
			// Recompute ProbT to account for new position before choosing a new direction
			// NOTE: this does not appear to help much, so leave it off for now
			//SlabZ = clamp(dot(DwivediSlabOrigin - Ray.Origin, DwivediSlabNormal), 0.0, SlabThickness);
			//ProbT = SlabThickness > 0.0 ? rcp(1 + exp(SigmaT * (SlabThickness - 2 * SlabZ) / DwivediScale)) : 0.0;
#endif

			float4 Result = SampleDwivediPhaseFunction(ColorChannelPdf, DwivediScale, GuidedRatio, ProbT, DwivediSlabNormal, Ray.Direction, G, RandSample.xy);
			Ray.Direction = Result.xyz;
			PathThroughput *= Result.w;
#else
			// importance sample the phase function
			float4 DirectionAndPhase = ImportanceSampleHenyeyGreensteinPhase(RandSample.xy, G);
			Ray.Direction = TangentToWorld(DirectionAndPhase.xyz, Ray.Direction);
			// NOTE: phase function cancels out since it is being perfectly importance sampled
			// account for transmittance and scattering up to this point
			PathThroughput *= SigmaS * EvaluateSpectralTransmittanceHit(Ray.TMax, SigmaT, ColorChannelPdf).xyz;
#endif
			// keep scattering
			continue;
		}
		else
		{
#if SSS_USE_DWIVEDI
            // account for transmittance to the boundary as well as the guiding probability
			PathThroughput *= EvaluateGuidedSpectralTransmittanceMiss(ProbeResult.HitT, SlabCosine, DwivediScale, GuidedRatio, SigmaT, ProbT, ColorChannelPdf).xyz;
#else
			// account for transmittance to the boundary
			PathThroughput *= EvaluateSpectralTransmittanceMiss(ProbeResult.HitT, SigmaT, ColorChannelPdf).xyz;
#endif
			// our short ray hit the geometry - decide if we should exit or not
#if SSS_USE_TIR
			float3 WorldNormal = ProbeResult.WorldNormal;
			float CosTheta = abs(dot(Ray.Direction, WorldNormal));
			float Fresnel = FresnelReflectance(CosTheta, 1.0 / 1.4);
			if (RandSample.x < Fresnel)
			{
				// internal reflection occured -- reflect and keep tracing
				// NOTE: weight and probability cancel out, so no throughput adjustment is needed
				Ray.Origin += ProbeResult.HitT * Ray.Direction;
				Ray.Direction = reflect(Ray.Direction, WorldNormal);

#if SSS_USE_DWIVEDI && SSS_USE_DWIVEDI_USE_THIN_SLABS
				// we hit the boundary, so reset the slab origin and look for a new one
				DwivediSlabOrigin = Ray.Origin;
				DwivediSlabNormal = ProbeResult.WorldSmoothNormal * ((ProbeResult.FrontFace != Payload.IsFrontFace()) ? -1.0 : 1.0);
				bDoSlabSearch = GuidedRatio > 0;
#endif

				ApplyRayBias(Ray.Origin, ProbeResult.HitT, ProbeResult.WorldGeoNormal);

				// Reset interface counter as if we had just entered the model (taking into account that we are seeing it from the opposite side now)
				InterfaceCounter = ProbeResult.FrontFace ? -1 : +1;
				// keep scattering
				continue;
			}
#endif // SSS_USE_TIR

			// we hit the boundary! overwrite the current payload and exit the walk with a diffuse scattering event
			Payload.TranslatedWorldPos = Ray.Origin + ProbeResult.HitT * Ray.Direction;
			// make sure normal is pointed outward from the object so we capture illumination from the exterior
			float SignFlip = (ProbeResult.FrontFace != Payload.IsFrontFace()) ? -1.0 : 1.0;
			Payload.WorldNormal       = SignFlip * ProbeResult.WorldNormal;
			Payload.WorldSmoothNormal = SignFlip * ProbeResult.WorldSmoothNormal;
			Payload.WorldGeoNormal    = SignFlip * ProbeResult.WorldGeoNormal;
			
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT; // diffuse shading
			Payload.BSDFOpacity = 1;
			Payload.SetBaseColor(1.0);
			Payload.TransparencyColor = 0;
			return true;
		}
	}
	// we scattered a bunch of times and never hit anything -- give up
	return false;
}

// Apply sagittal/tangential Petzval lens effect
//  BokehCenterUV: Viewport UV of the center of the bokeh
//  SamplePos: UV of the sample relative to the bokeh center, with 1 = bokeh boundary
float2 ApplyPetzval(float2 BokehCenterUV, float2 SamplePos)
{
	if (Petzval != 0)
	{
		float2 BokehCenter = BokehCenterUV * 2.0 - 1.0;
		
		BokehCenter.y *= -1.0;

		const float Radius = PetzvalExclusionBoxRadius * min(PetzvalExclusionBoxExtents.x, PetzvalExclusionBoxExtents.y);
		const float2 BoxToBokeh = sign(BokehCenter) * max(abs(BokehCenter) - PetzvalExclusionBoxExtents + Radius, 0.0);
		float BokehDistance = max(0.0, length(BoxToBokeh));
		const float2 Normal = BoxToBokeh / BokehDistance;
		BokehDistance = max(0.0, BokehDistance - Radius);


		if (BokehDistance > 0)
		{
			const float2 Tangent = float2(Normal.y, -Normal.x);

			const float Intensity = pow(BokehDistance, PetzvalFalloffPower);
			const float Stretch = rcp(1 + abs(Petzval) * Intensity);

			float2x2 ToSaggitalSpace = { Normal.x, Normal.y, Tangent.x, Tangent.y };
			
			float2 SagittalPos = mul(SamplePos, ToSaggitalSpace);
			bool bMask = Petzval > 0;
			SagittalPos.x *=  bMask ? Stretch : 1.0; // Sagittal
			SagittalPos.y *= !bMask ? Stretch : 1.0; // Tangential
			return mul(SagittalPos, transpose(ToSaggitalSpace));
		}
	}
	return SamplePos;
}

FPathState CreatePathState(int2 PixelIndex, int2 TextureIndex)
{
	FPathState Output = (FPathState)0;

	uint2 LaunchIndex = PixelIndex + View.ViewRectMin.xy;

#if PATH_TRACER_USE_ADAPTIVE_SAMPLING
	// initialize progressive sobol sequence based on current index for this pixel
	// NOTE: Error diffusion sampler cannot be used in this mode
	int SampleIndex = int(VarianceTexture[TextureIndex].z);
	Output.RandSequence = RandomSequenceCreate(LaunchIndex.x + LaunchIndex.y * 65536, TemporalSeed - Iteration + SampleIndex);
#else
	// Initialize random sequence 
	if (SamplerType == PATHTRACER_SAMPLER_ERROR_DIFFUSION)
	{
		// z-sampler init
		Output.RandSequence = RandomSequenceCreate(uint3(LaunchIndex, TemporalSeed - Iteration), Iteration, MaxSamples);
	}
	else
	{
		// random sobol init
		Output.RandSequence = RandomSequenceCreate(LaunchIndex.x + LaunchIndex.y * 65536, TemporalSeed);
	}
#endif

	// Initialize ray and payload
	float4 Jitter = Output.RandSequence.Get4D();
	float2 ViewportUV;
	{
		float2 AAJitter = Jitter.xy;
#if 0
		// Tent filter
		float2 P = 2 * AAJitter - 0.99999994f;
		AAJitter = 0.5 + sign(P) * (1.0 - sqrt(1.0 - abs(P)));
#elif 1
		// importance sample a gaussian kernel with variable sigma
		float3 Disk = ConcentricDiskSamplingHelper(AAJitter);
		float Sigma = FilterWidth / 6.0; // user-provided width covers +/-3*Sigma
		AAJitter = 0.5 + Sigma * Disk.xy * sqrt(-2.0 * log(1.0 - Disk.z * Disk.z));
#endif
		ViewportUV = (PixelIndex + AAJitter) * View.ViewSizeAndInvSize.zw;
		float2 JitteredViewportUV = (LaunchIndex + AAJitter) * View.BufferSizeAndInvSize.zw;
		Output.Ray = CreatePrimaryRay(JitteredViewportUV);
	}

	if (CameraLensRadius.y > 0)
	{
	    // DOF enabled - apply simple thin-lens model
		float2 LensSample = Jitter.zw;
		float3 ViewX = View.ViewToTranslatedWorld[0].xyz;
		float3 ViewY = View.ViewToTranslatedWorld[1].xyz;
		float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
		// shift back origin by the near plane amount
		float ZFactor = rcp(dot(ViewZ, Output.Ray.Direction));
		float3 NearNudge = (View.NearPlane * ZFactor) * Output.Ray.Direction;
		float3 Origin = Output.Ray.Origin - NearNudge;
		// compute focus plane
		float3 FocusP = Origin + (CameraFocusDistance * ZFactor) * Output.Ray.Direction;
		// nudge ray origin
		LensSample = UniformSampleDiskConcentric(LensSample);
		LensSample = ApplyPetzval(ViewportUV, LensSample);
		LensSample = CameraLensRadius * LensSample;
		Origin += LensSample.x * ViewX + LensSample.y * ViewY;
		// recompute direction
		Output.Ray.Direction = normalize(FocusP - Origin);
		// move ray origin back to the near plane for consistency
		Output.Ray.Origin = Origin + Output.Ray.Direction * (View.NearPlane * rcp(dot(ViewZ, Output.Ray.Direction)));
	}

	// path state variables (these cary information between bounces)
	Output.PathThroughput = 1.0;
	Output.SetPathRoughness(0.0);
	Output.SetSigmaT(float3(StartingMediumData[0], StartingMediumData[1], StartingMediumData[2]));
	Output.SetSigmaSAndPhase(float3(StartingMediumData[3], StartingMediumData[4], StartingMediumData[5]), StartingMediumData[6]);
	Output.SetFirstScatterType(PATHTRACER_SCATTER_CAMERA);

	#if DEBUG_ENABLE && PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	Output.PixelCoord = LaunchIndex;
	#endif

	return Output;
}

bool PathTracingKernel(inout FPathState PathState, int Bounce)
{
	// This array will hold a CDF for light picking
	float LightPickingCdf[RAY_TRACING_LIGHT_COUNT_MAXIMUM];
	int LightPickingIds[RAY_TRACING_LIGHT_COUNT_MAXIMUM];

#if 0
	// visualize the shadow ray handling directly
	PathState.Radiance = TraceTransparentVisibilityRay(PathState.Ray, 0.0, 0.0, 0, true, true, PathState.RandSequence);
	return false;
#endif

	FVolumeSegment VolumeSegment = CreateEmptyVolumeSegment();
	FPackedPathTracingPayload PackedPayload = TraceTransparentRay(
		PathState,
		Bounce,
		VolumeSegment);

	// process the returned volume segment if we got one
	if (VolumeSegment.IsValid() && VolumeMISMode != 0)
	{
		const float3 Ro = PathState.Ray.Origin;
		const float3 Rd = PathState.Ray.Direction;
		const float VTMin = VolumeSegment.Interval.VolumeTMin;
		const float VTMax = VolumeSegment.Interval.VolumeTMax;

		float LightPickingCdfSum = 0.0;
		int NumLights = 0;
		for (int LightId = SceneInfiniteLightCount; LightId < SceneLightCount; LightId++)
		{
			FVolumeLightSampleSetup LightSetup = PrepareLightVolumeSample(LightId, Ro, Rd, VTMin, VTMax);
			if (LightSetup.IsValid())
			{
				float LightProb = LightSetup.LightImportance * GetVolumetricScatteringIntensity(LightId);
				if (LightProb > 0)
				{
					LightPickingCdfSum += LightProb;
					LightPickingCdf[NumLights] = LightPickingCdfSum;
					LightPickingIds[NumLights] = LightId;
					NumLights++;
					if (NumLights == RAY_TRACING_LIGHT_COUNT_MAXIMUM)
					{
						// reached max
						break;
					}
				}
			}
		}

		if (LightPickingCdfSum > 0.0)
		{
			// at least one light is overlapping with our ray, so we have a chance to sample it
			float2 RandSample = PathState.RandSequence.Get2D();
			int LightSampleLightId = 0;
			float LightPickPdf = 0.0;
			SelectLight(RandSample.x * LightPickingCdfSum, NumLights, LightPickingCdf, LightSampleLightId, LightPickPdf);
			LightSampleLightId = LightPickingIds[LightSampleLightId]; // map back to original LightId
			LightPickPdf /= LightPickingCdfSum;

			// picked a light! now use the equi-angular sampler to pick a position along the ray and store it for later
			// so that we can compute the light during the ray-marching loop, when the path prefix pdf will be known
			FVolumeLightSampleSetup LightSetup = PrepareLightVolumeSample(LightSampleLightId, Ro, Rd, VTMin, VTMax);
			// Should be safe to assume LightSetup.IsValid() is true because otherwise the pdf would have been 0
			float2 SampleResult = LightSetup.SampleDistance(RandSample.y);
			float EquiAngularT = SampleResult.x;
			float EquiAngularPathPdf = SampleResult.y;


			// Account for the transmittance up to the current point within the current slice of volume
			VolumeSegment.Interval.VolumeTMax = EquiAngularT;
			float CloudFactor = 1.0;
			const FVolumeShadedResult Result = VolumeGetDensityAndTransmittance(Ro, Rd, VolumeSegment.Interval, Bounce, PathState.RandSequence, VolumeSegment.Throughput, CloudFactor);
			float3 Contrib = VolumeSegment.Throughput;
			if (any(Contrib > 0))
			{
				// account for probability of the path prefix
				Contrib /= EquiAngularPathPdf * LightPickPdf;

				// find out how much volume exists at the current point
				const float3 TranslatedWorldPos = Ro + EquiAngularT * Rd;

				// prepare a minimal payload that describes the hit we need to shade
				FPathTracingPayload VolPayload = CreateMediumHitPayload(EquiAngularT, TranslatedWorldPos, Result);
				float3 LightRandValue = PathState.RandSequence.Get3D();

				const bool bCastShadows = CastsVolumeShadow(LightSampleLightId);
				const bool bCloudCastShadows = true; // volumes should get shadows from clouds

				const uint MissShaderIndex = GetLightMissShaderIndex(LightSampleLightId);
				// compute direct light sampling?
				if (MISMode != 0)
				{
					FLightSample LightSample = SampleLight(LightSampleLightId, LightRandValue.xy, TranslatedWorldPos, float3(0, 0, 0));
					if (LightSample.Pdf > 0)
					{
						FRayDesc LightRay;
						LightRay.Origin = TranslatedWorldPos;
						LightRay.TMin = 0;
						LightRay.Direction = LightSample.Direction;
						LightRay.TMax = LightSample.Distance;
						if (any(LightSample.RadianceOverPdf > 0))
						{
							// Evaluate material
							FMaterialEval MaterialEval = Medium_EvalMaterial(-Rd, LightSample.Direction, VolPayload, float2(1.0, 0.0));

							// Record the contribution
							float3 LightContrib = Contrib * LightSample.RadianceOverPdf * GetVolumetricScatteringIntensity(LightSampleLightId) * MaterialEval.Weight * MaterialEval.Pdf;
							if (MISMode == 2)
							{
								LightContrib *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
							}
							LightContrib *= TraceTransparentVisibilityRay(LightRay, 1.0, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows, bCloudCastShadows, CloudFactor, PathState.RandSequence);

							PathState.AccumulateRadiance(LightContrib);
						}
					}
				}
				// now compute again with phase function MIS
				if (MISMode != 1)
				{
					FMaterialSample MaterialSample = Medium_SampleMaterial(-Rd, VolPayload, LightRandValue);
					if (MaterialSample.Pdf > 0)
					{
						FRayDesc MaterialRay;
						MaterialRay.Origin = TranslatedWorldPos;
						MaterialRay.Direction = MaterialSample.Direction;
						MaterialRay.TMin = 0.0;
						MaterialRay.TMax = RAY_DEFAULT_T_MAX;
						FLightHit LightResult = TraceLight(MaterialRay, LightSampleLightId);
						if (LightResult.IsHit())
						{
							float3 LightContrib = Contrib * MaterialSample.Weight * LightResult.Radiance * GetVolumetricScatteringIntensity(LightSampleLightId);
							if (MISMode == 2)
							{
								LightContrib *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf);
							}
							MaterialRay.TMax = LightResult.HitT;
							LightContrib *= TraceTransparentVisibilityRay(MaterialRay, 1.0, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows, bCloudCastShadows, CloudFactor, PathState.RandSequence);
							PathState.AccumulateRadiance(LightContrib);
						}
					}
				}
			}
		}
	}

	if (PackedPayload.IsMiss())
	{
		// we didn't hit anything selectable for further shading, we are done
		return false;
	}

	FPathTracingPayload Payload = UnpackPathTracingPayload(PackedPayload, PathState.Ray);
	AdjustPayloadAfterUnpack(Payload, ApplyDiffuseSpecularOverrides);

	// As soon as the path is blurry enough, we can get away with diffuse sampling only
	const bool bSimplifySSS = PathState.GetPathRoughness() >= 0.15;

	// Processing the random walk will (stochastically) move the shading point elsewhere on the surface of the object
	if (!ProcessSubsurfaceRandomWalk(Payload, PathState.PathThroughput, PathState.RandSequence, PathState.Ray.Direction, bSimplifySSS))
	{
		// random walk did not terminate at a valid point
		return false;
	}

	// Don't treat internal volume scattering as a volume sample (so its not affected by those options in the lights)
	const bool bIsVolumeSample = Payload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_MEDIUM && !any(PathState.GetSigmaT() > 0);

	FLightLoopCount LightLoopCount = LightGridLookup(Payload.TranslatedWorldPos);
	// If this is a volume sample, we have not forced the renderer to ignore volume segment sampling _and_ we aren't in an internal medium which does not use volume segments
	if (bIsVolumeSample && VolumeMISMode != 0)
	{
		// if we are using the volume segment for local lights, exclude them from the light loop below
		LightLoopCount.NumLights = SceneInfiniteLightCount;
		LightLoopCount.NumMISLights = SceneInfiniteLightCount;
	}

	// Choose a random number for both Light sampling and BxDF sampling
	float4 RandSample = PathState.RandSequence.Get4D();

	const float2 DiffuseSpecularScale = PathState.GetDiffuseSpecularScale(bIsVolumeSample);

	const bool bDoLightLoop = any(DiffuseSpecularScale > 0);

	float LightPickingCdfSum = 0;

	// Sample material
	FMaterialSample MaterialSample = SampleMaterial(-PathState.Ray.Direction, Payload, RandSample.xyz);

	// Debug Printing
	#if DEBUG_ENABLE && PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	FShaderPrintContext Ctx = InitShaderPrintContextAtCursor(PathState.PixelCoord, uint2(50, 50));
	if (Ctx.bIsActive)
	{
		Print(Ctx, TEXT("Debug"), FontRed); Newline(Ctx);
		Print(Ctx, TEXT("Pixel Coord : "), FontWhite); Print(Ctx, PathState.PixelCoord, FontYellow); Newline(Ctx);

		// Example of debugging :
		#if SUBSTRATE_PATH_TRACING_GLINTS
		PrintLineN(Ctx, Payload.GlintValue);
		PrintLineN(Ctx, Payload.GlintUV);
		PrintLineN(Ctx, Payload.GlintUVdx);
		PrintLineN(Ctx, Payload.GlintUVdy);
		#endif
	}
	#endif

	// If we are using Light sampling and the material can use it ...
	if (MISMode != 0 && bDoLightLoop)
	{
		// Choose a light and sample it
		float3 TranslatedWorldPos = Payload.TranslatedWorldPos;
		float3 WorldNormal = Payload.WorldNormal;
		uint PrimitiveLightingChannelMask = Payload.PrimitiveLightingChannelMask;

		const bool bIsTransmissiveMaterial = ENABLE_TRANSMISSION && Payload.IsMaterialTransmissive();

		for (uint Index = 0, Num = LightLoopCount.NumLights; Index < Num; ++Index)
		{
			uint LightIndex = GetLightId(Index, LightLoopCount);
			float LightEstimate = EstimateLight(LightIndex, TranslatedWorldPos, WorldNormal, PrimitiveLightingChannelMask, bIsTransmissiveMaterial);
			if (bIsVolumeSample)
			{
				LightEstimate *= GetVolumetricScatteringIntensity(LightIndex);
			}
			if (PathState.HasMadeDiffuseOrSpecularScatter())
			{
				LightEstimate *= GetLightIndirectScale(LightIndex, PathState.GetPathRoughness());
			}
			LightPickingCdfSum += LightEstimate;
			LightPickingCdf[Index] = LightPickingCdfSum;
		}

		if (LightPickingCdfSum > 0)
		{
			// init worked
			int LightId;
			float LightPickPdf = 0;

			SelectLight(RandSample.z * LightPickingCdfSum, LightLoopCount.NumLights, LightPickingCdf, LightId, LightPickPdf);
			LightPickPdf /= LightPickingCdfSum;

			LightId = GetLightId(LightId, LightLoopCount);
			FLightSample LightSample = SampleLight(LightId, RandSample.xy, TranslatedWorldPos, WorldNormal);

			LightSample.RadianceOverPdf /= LightPickPdf;
			LightSample.Pdf *= LightPickPdf;
			if (LightSample.Pdf > 0)
			{
				// for transmissive materials, bias the position to the other side of the surface if the light is coming from behind
				const float SignedPositionBias = bIsTransmissiveMaterial ? sign(dot(Payload.WorldGeoNormal, LightSample.Direction)) : 1.0;
				FRayDesc LightRay;
				LightRay.Origin = TranslatedWorldPos;
				LightRay.TMin = 0;
				LightRay.Direction = LightSample.Direction;
				LightRay.TMax = LightSample.Distance;
				ApplyRayBias(LightRay.Origin, Payload.HitT, SignedPositionBias * Payload.WorldGeoNormal);

				float AvgRoughness = ApproximateCaustics * GetAverageRoughness(Payload);

				if (bIsVolumeSample)
				{
					LightSample.RadianceOverPdf *= GetVolumetricScatteringIntensity(LightId);
				}

				if (PathState.HasMadeDiffuseOrSpecularScatter())
				{
					LightSample.RadianceOverPdf *= GetLightIndirectScale(LightId, PathState.GetPathRoughness());
				}

				// #dxr_todo: Is it cheaper to fire the ray first? Or eval the material first?
				if (any(LightSample.RadianceOverPdf > 0))
				{
					// Evaluate material
					float2 LightDiffuseSpecularScale = DiffuseSpecularScale * float2(GetLightDiffuseScale(LightId), GetLightSpecularScale(LightId));

					FMaterialEval MaterialEval = EvalMaterial(-PathState.Ray.Direction, LightSample.Direction, Payload, LightDiffuseSpecularScale);

					// Record the contribution
					float3 LightContrib = PathState.PathThroughput * LightSample.RadianceOverPdf * MaterialEval.Weight * MaterialEval.Pdf;
					if (MISMode == 2)
					{
						LightContrib *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
					}
					const bool bCastShadows = bIsVolumeSample ? CastsVolumeShadow(LightId) : CastsShadow(LightId);
					const bool bCloudCastShadows = bIsVolumeSample ? true : CastsCloudShadow(LightId);
					const float CloudFactor = bIsVolumeSample ? Payload.GetCloudFactor() : 1.0;
					const uint MissShaderIndex = GetLightMissShaderIndex(LightId);
					LightContrib *= TraceTransparentVisibilityRay(LightRay, AvgRoughness, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows, bCloudCastShadows, CloudFactor, PathState.RandSequence);

					PathState.AccumulateRadiance(LightContrib);
				}
			}
		}
	}

	if (asuint(MaterialSample.Pdf) > 0x7F800000)
	{
		// Pdf became invalid (either negative or NaN)
		PathState.Radiance = float3(1, 0, 1);
		return false;
	}

	if (!(MaterialSample.Pdf > 0))
	{
		// No valid direction -- we are done
		return false;
	}

	float3 NextPathThroughput = PathState.PathThroughput * MaterialSample.Weight;
	if (!any(NextPathThroughput > 0))
	{
		// no energy left in this path
		return false;
	}

	// Russian roulette:
	//   The probability of keeping the path should be roughly proportional to the weight at the current shade point,
	//  but just using MaterialWeight would miss out on cases where the path throughput changes color (like in a cornell
	//  box when bouncing between walls of different colors). So use the ratio of the brightest color channel in the
	//  previous and next throughput.
	//   The second tweak is to add a sqrt() around the probability to soften the termination probability (paths will last
	//  a little longer). This allows paths to go a bit deeper than the naive heuristic while still allowing them to terminate
	//  early. This makes RR effective from the very first bounce without needing to delay it.
	float ContinuationProb = sqrt(saturate(max3(NextPathThroughput.x, NextPathThroughput.y, NextPathThroughput.z) / max3(PathState.PathThroughput.x, PathState.PathThroughput.y, PathState.PathThroughput.z)));
	if (ContinuationProb < 1)
	{
		// If there is some chance we should terminate the ray, draw an extra random value
		float RussianRouletteRand = RandSample.w;
		if (RussianRouletteRand >= ContinuationProb)
		{
			// stochastically terminate the path
			return false;
		}
		PathState.PathThroughput = NextPathThroughput / ContinuationProb;
	}
	else
	{
		PathState.PathThroughput = NextPathThroughput;
	}

	// Update ray according to material sample
	PathState.Ray.Origin = Payload.TranslatedWorldPos;
	PathState.Ray.Direction = MaterialSample.Direction;
	PathState.Ray.TMin = 0;
	PathState.Ray.TMax = RAY_DEFAULT_T_MAX;
	ApplyRayBias(PathState.Ray.Origin, Payload.HitT, MaterialSample.PositionBiasSign * Payload.WorldGeoNormal);

	// enlarge roughness based on the chosen lobe roughness
	float PreviousPathRoughness = PathState.GetPathRoughness();
	PathState.SetPathRoughness(max(PreviousPathRoughness, MaterialSample.Roughness));

	// update the current extinction if we are crossing a boundary on glass or water
	// summing the local extinction gives a rudimentary way of dealing with overlapping regions
	// long term we will probably want a stack with priorities
	if (MaterialSample.PositionBiasSign < 0 && Payload.IsMaterialSolidGlass())
	{
		const float3 LocalSigmaT = Payload.GetExtinction();
		const float3 LocalSigmaS = Payload.GetGlassAlbedo() * LocalSigmaT;
		const float LocalPhase = Payload.GetGlassPhase();
		float3 SigmaT = PathState.GetSigmaT();
		float4 ScatterData = PathState.GetSigmaSAndPhase();
		float WeightPrev = max(LobeColorToWeight(SigmaT), 0);
		float WeightCurr = max(LobeColorToWeight(LocalSigmaT), 0);
		if (Payload.IsFrontFace())
		{
			// entering
			SigmaT += LocalSigmaT;
			ScatterData.xyz += LocalSigmaS;
		}
		else
		{
			// exiting
			SigmaT -= LocalSigmaT;
			SigmaT = max(SigmaT, 0);
			ScatterData.xyz -= LocalSigmaS;
		}
		ScatterData.xyz = clamp(ScatterData.xyz, 0, LocalSigmaT);
		ScatterData.w = lerp(ScatterData.w, LocalPhase, MISWeightBalanced(WeightCurr, WeightPrev));
		PathState.SetSigmaT(SigmaT);
		PathState.SetSigmaSAndPhase(ScatterData.xyz, ScatterData.w);
	}

	// If we are using Material sampling for lights
	if (MISMode != 1 && bDoLightLoop)
	{
		// Check which lights can be seen by the material ray and trace a dedicated shadow ray
		// While it would be possible to just loop around and use the indirect ray to do this, it would prevent the application
		// of shadow ray specific logic for transparent shadows or various per light tricks like shadow casting
		const bool bUseMIS = MISMode == 2 && LightPickingCdfSum > 0;
		float PreviousCdfValue = 0.0;
		for (uint Index = 0, Num = LightLoopCount.NumMISLights; Index < Num; ++Index)
		{
			const uint LightId = GetLightId(Index, LightLoopCount);
			const float CdfValue = LightPickingCdf[Index];
			const float LightPickPdf = (CdfValue - PreviousCdfValue) / LightPickingCdfSum;
			PreviousCdfValue = CdfValue;
			if ((Payload.PrimitiveLightingChannelMask & GetLightingChannelMask(LightId)) == 0)
			{
				// light does not affect the current ray
				continue;
			}

			FLightHit LightResult = TraceLight(PathState.Ray, LightId);

			if (LightResult.IsMiss())
			{
				continue;
			}

			float3 LightContrib = PathState.PathThroughput * LightResult.Radiance;
			switch (MaterialSample.ScatterType)
			{
				case PATHTRACER_SCATTER_DIFFUSE:
				{
					LightContrib *= GetLightDiffuseScale(LightId) * DiffuseSpecularScale.x;
					break;
				}
				case PATHTRACER_SCATTER_SPECULAR:
				case PATHTRACER_SCATTER_REFRACT:
				{
					LightContrib *= GetLightSpecularScale(LightId) * DiffuseSpecularScale.y;
					break;
				}
				case PATHTRACER_SCATTER_VOLUME:
				{
					LightContrib *= GetVolumetricScatteringIntensity(LightId) * DiffuseSpecularScale.x;
					break;
				}
			}

			if (bUseMIS)
			{
				LightContrib *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf * LightPickPdf);
			}

			if (PathState.HasMadeDiffuseOrSpecularScatter())
			{
				LightContrib *= GetLightIndirectScale(LightId, PreviousPathRoughness);
			}

			if (any(LightContrib > 0))
			{
				FRayDesc LightRay = PathState.Ray;
				LightRay.TMax = LightResult.HitT;
				const bool bCastShadows = bIsVolumeSample ? CastsVolumeShadow(LightId) : CastsShadow(LightId);
				const bool bCloudCastShadows = bIsVolumeSample ? true : CastsCloudShadow(LightId);
				const float CloudFactor = bIsVolumeSample ? Payload.GetCloudFactor() : 1.0;
				const uint MissShaderIndex = GetLightMissShaderIndex(LightId);
                const float ShadowRayRoughness = ApproximateCaustics * PathState.GetPathRoughness();
				LightContrib *= TraceTransparentVisibilityRay(LightRay, ShadowRayRoughness, PreviousPathRoughness, MissShaderIndex, bCastShadows, bCloudCastShadows, CloudFactor, PathState.RandSequence);
				PathState.AccumulateRadiance(LightContrib);
			}
		}
	}

#if PATH_TRACER_USE_CLOUD_SHADER
	if (bIsVolumeSample && Payload.GetCloudFactor() < 1)
	{
		// Cloud MS approximation octaves should not bounce
		return false;
	}
#endif

	// continue the path only if the relevant indirect contribution is enabled
	bool bKeepGoing = PathState.UpdateScatterType(MaterialSample.ScatterType);
	// also check for last bounce if emissive contributions are not needed
	return bKeepGoing && (Bounce + 1 < MaxBounces || PathState.ShouldAccumulateEmissive());
}
