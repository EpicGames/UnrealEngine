// Copyright Epic Games, Inc. All Rights Reserved.

#define RANDSEQ_SFC_TEXTURE 1

#define LIGHT_LOOP_METHOD           2 // 0: stochastic (like ref PT) 1: loop over all lights 2: RIS (take 1 sample per light, but only keep one to trace a single shadow ray)

#define EVALUATE_MATERIAL_BSDF      1 // Use the full material eval/sample when true, otherwise substitute a plain lambert calculation for speed
#define USE_BSDF_SAMPLING           1 // 0: light sampling only (no MIS) 1: light sampling and BSDF sampling (with MIS)

#if LIGHT_LOOP_METHOD == 0
#define LIGHT_SELECTION_METHOD      1 // 0: uniform sampling, 1: CDF (keeps stratification), 2: RIS (destroys stratification but no array needed)
#define BSDF_SAMPLING_METHOD        0 // 0: trace all, 1: use RIS among all valid light hits (seems slower (!))
#endif


#if LIGHT_LOOP_METHOD == 2
#define ACCUMULATE_UNSHADOWED_LIGHTS 1 // When true, in LIGHT_LOOP_METHOD == 2, accumulate any unshadowed light directly, skipping RIS to reduce noise
#define FORCE_TRACE_INFINITE_LIGHTS  0 // When true, in LIGHT_LOOP_METHOD == 2, trace a ray for infinite lights instead of letting them do RIS
#endif

#define CLAMP_PATH_INTENSITY        1 // Apply MaxPathIntensity to limit intensity of indirect paths like the reference path tracer

#define USE_PATH_TRACING_LIGHT_GRID	1
#define USE_RAY_TRACING_DECAL_GRID	1

#ifndef PATH_TRACER_PRIMARY_RAYS            // Should be set by permutation (set for editor visibility)
#define PATH_TRACER_PRIMARY_RAYS    1
#endif

#ifndef PATH_TRACER_INCLUDE_TRANSLUCENT     // Should be set by permutation (set for editor visibility)
#define PATH_TRACER_INCLUDE_TRANSLUCENT   1 // include objects tagged as TRANSLUCENT when tracing rays?
#endif


#include "../Common.ush"
#include "../RayTracing/RayTracingCommon.ush"
#include "../RayTracing/RayTracingDecalGrid.ush"
#include "PathTracingCommon.ush"
#include "PathTracingDecals.ush"
#include "Material/PathTracingMaterialSampling.ush"
#include "Utilities/PathTracingRandomSequence.ush"
#include "Utilities/PathTracingRIS.ush"
#include "Light/PathTracingLightSampling.ush"
#include "Light/PathTracingLightGrid.ush"

#if PATH_TRACER_USE_SER
#include "/Engine/Public/NvidiaExtensionsCommon.ush"
#endif

RWTexture2D<float4> RWSceneColor;
RWTexture2D<float4> RWSceneDiffuseColor;
RWTexture2D<float4> RWSceneSpecularColor;
RWTexture2D<float4> RWSceneNormal;
RWTexture2D<float>  RWSceneRoughness;
RWTexture2D<float>  RWSceneLinearDepth;
RWTexture2D<float>  RWSceneDepth;
RaytracingAccelerationStructure TLAS;
uint SceneVisibleLightCount;

int DebugMode;
int SamplesPerPixel;
int SampleIndex;
float SampleBlendFactor;
int MaxBounces;
int EnableEmissive;
#if CLAMP_PATH_INTENSITY
float MaxPathIntensity;
#endif
#if PATH_TRACER_PRIMARY_RAYS == 0
float AmbientOcclusionMaxDistance;
uint VisualizeMaxCount;
#endif
#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
int FirstBounce;
#endif

uint4 GridHash(uint4 P)
{
	const uint4 S = uint4(16, 17, 18, 19);
	const uint K = 1159349557u;
	P ^= P >> S; P *= K; P ^= P.yzwx ^ P.zwxy;
	P ^= P >> S; P *= K; P ^= P.yzwx ^ P.zwxy;
	P ^= P >> S; P *= K; P ^= P.yzwx ^ P.zwxy;
	return P;
}

float3 TraceShadowRay(FRayDesc Ray, float PathRoughness, float PreviousPathRoughness, uint MissShaderIndex, bool bCastShadows)
{
	if (!bCastShadows && MissShaderIndex == 0)
	{
		// no work to do
		return 1.0;
	}

	FPackedPathTracingPayload PackedPayload = InitPathTracingVisibilityPayload(PathRoughness);

	if (!bCastShadows)
	{
		// ray should not cast shadows, make it degenerate so we can still run the miss shader
		Ray.TMin = POSITIVE_INFINITY;
		Ray.TMax = POSITIVE_INFINITY;
	}
	const uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER | GetPathTracingHairRayFlags(Ray.Origin);
	const uint InstanceInclusionMask = PATHTRACER_MASK_SHADOW;
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RAY_TRACING_SHADER_SLOT_SHADOW,
		RAY_TRACING_NUM_SHADER_SLOTS,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);

	if (PackedPayload.IsHit())
	{
		// We didn't run the miss shader, therefore we must have hit something opaque (or reached full opacity)
		return 0.0;
	}
	return PackedPayload.GetRayThroughput() * exp(-max(PackedPayload.GetTau(), 0.0));
}

struct FPathTracingResult {
	float4 Radiance;
	float3 DiffuseColor;
	float3 SpecularColor;
	float3 WorldNormal;
	float Roughness;
	float LinearDepth;
	float Z;
};

FPathTracingResult MakeResult(float3 Radiance, float Alpha, float Z)
{
	FPathTracingResult Result = (FPathTracingResult) 0;
	Result.Radiance.rgb = Radiance;
	Result.Radiance.a = Alpha;
	Result.Z = Z;
	return Result;
}

// TODO: Expose these like for the normal path tracer
static const uint EnabledDirectLightingContributions   = 15u;
static const uint EnabledIndirectLightingContributions = 15u;

struct FPathState
{                                   	// packed size
	FRandomSequence RandSequence;   	//  8 bytes
	float3 Radiance;                	// 12 bytes
	float Alpha;                   		//  4 bytes
	uint3 PackedDiffSpecColor;			// 12 bytes
	uint2 PackedNormalRoughness;		//  8 bytes
	FRayDesc Ray;                   	// 12+12 = 24 bytes (TMin/TMax do not need to be stored)
	float3 PathThroughput;          	// 12 bytes
	uint2 PackedRoughnessSigma;     	// 2 bytes // half precision (PathRoughness)
	// 6 bytes // half precision  (SigmaT)
	// 3 bits - packed in sign bits of SigmaT (FirstScatterType)
	uint2 PackedScatterPhase;       	// 6 bytes // half precision (SigmaS)
										// 2 bytes // phase g (could be quantized further)

	// Temporary parameters not stored
	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	float DepthZ;                   // Camera ray depth (don't need it otherwise)
	#endif

	float3 GetDiffuseColor()
	{
		return float3(UnpackFloat2FromUInt(PackedDiffSpecColor.x),
					  UnpackFloat2FromUInt(PackedDiffSpecColor.y).x);
	}

	float3 GetSpecularColor()
	{
		return float3(UnpackFloat2FromUInt(PackedDiffSpecColor.y).y,
					  UnpackFloat2FromUInt(PackedDiffSpecColor.z));
	}

	float3 GetNormal()
	{
		return float3(UnpackFloat2FromUInt(PackedNormalRoughness.x),
		              UnpackFloat2FromUInt(PackedNormalRoughness.y).x);
	}

	float GetRoughness()
	{
		return UnpackFloat2FromUInt(PackedNormalRoughness.y).y;
	}

	void SetDiffuseSpecularColor(float3 Diffuse, float3 Specular)
	{
		PackedDiffSpecColor.x = PackFloat2ToUInt(Diffuse.xy);
		PackedDiffSpecColor.y = PackFloat2ToUInt(float2(Diffuse.z, Specular.x));
		PackedDiffSpecColor.z = PackFloat2ToUInt(Specular.yz);
	}

	void SetNormal(float3 Normal)
	{
		PackedNormalRoughness.x = PackFloat2ToUInt(Normal.xy);
		PackedNormalRoughness.y = PackFloat2ToUInt(float2(Normal.z, GetRoughness()));
	}

	void SetRoughness(float Roughness)
	{
		PackedNormalRoughness.y = PackFloat2ToUInt(float2(GetNormal().z, Roughness));
	}

	uint GetFirstScatterType()
	{
		// Extract SigmaT sign bits
		uint a = (PackedRoughnessSigma.x >> 29) & 4u;
		uint b = (PackedRoughnessSigma.y >> 30) & 2u;
		uint c = (PackedRoughnessSigma.y >> 15) & 1u;
		return a | b | c;
	}

	void SetFirstScatterType(uint ScatterType)
	{
		uint a = ScatterType & 4u;
		uint b = ScatterType & 2u;
		uint c = ScatterType & 1u;
		// erase old value
		PackedRoughnessSigma.x &= 0x7fffffffu;
		PackedRoughnessSigma.y &= 0x7fff7fffu;
		// set new value
		PackedRoughnessSigma.x |= a << 29;
		PackedRoughnessSigma.y |= b << 30;
		PackedRoughnessSigma.y |= c << 15;
	}

	float GetPathRoughness()
	{
		return f16tof32(PackedRoughnessSigma.x);
	}

	void SetPathRoughness(float PathRoughness)
	{
		PackedRoughnessSigma.x &= 0xffff0000u;
		PackedRoughnessSigma.x |= f32tof16(PathRoughness);
	}
	
	float3 GetSigmaT()
	{
		return abs(float3(f16tof32(PackedRoughnessSigma.x >> 16),
		                  f16tof32(PackedRoughnessSigma.y),
		                  f16tof32(PackedRoughnessSigma.y >> 16)));
	}

	void SetSigmaT(float3 SigmaT)
	{
		// erase old value (keeping sign bits)
		PackedRoughnessSigma.x &= 0x8000ffffu;
		PackedRoughnessSigma.y &= 0x80008000u;
		SigmaT = abs(SigmaT); // make sure there are no sign bits
		// write new value
		PackedRoughnessSigma.x |= f32tof16(SigmaT.x) << 16;
		PackedRoughnessSigma.y |= f32tof16(SigmaT.y);
		PackedRoughnessSigma.y |= f32tof16(SigmaT.z) << 16;
	}

	float4 GetSigmaSAndPhase()
	{
		return float4(UnpackFloat2FromUInt(PackedScatterPhase.x), UnpackFloat2FromUInt(PackedScatterPhase.y));
	}

	void SetSigmaSAndPhase(float3 SigmaS, float PhaseG)
	{
		PackedScatterPhase.x = PackFloat2ToUInt(SigmaS.xy);
		PackedScatterPhase.y = PackFloat2ToUInt(float2(SigmaS.z, PhaseG));
	}

	bool HasMadeContributionScatter()
	{
		// The first scattering type that matters for the PATHTRACER_CONTRIBUTION_* flags is the first non-refractive one
		// We choose to track camera and pure refracted paths together so that objects behind glass can be treated the same as directly visible objects.
		uint FirstScatterType = GetFirstScatterType();
		return !(FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
			     FirstScatterType == PATHTRACER_SCATTER_REFRACT);
	}

	bool HasMadeDiffuseOrSpecularScatter()
	{
		uint FirstScatterType = GetFirstScatterType();
		return FirstScatterType == PATHTRACER_SCATTER_DIFFUSE || FirstScatterType == PATHTRACER_SCATTER_SPECULAR;
	}

    uint GetInstanceInclusionMaskOpaque()
    {
        uint FirstScatterType = GetFirstScatterType();
        if (FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
            FirstScatterType == PATHTRACER_SCATTER_REFRACT)
        {
            return PATHTRACER_MASK_CAMERA_OPAQUE;
        }
        if (FirstScatterType == PATHTRACER_SCATTER_SPECULAR)
        {
            return PATHTRACER_MASK_REFLECTION_OPAQUE;
        }
        return PATHTRACER_MASK_INDIRECT_OPAQUE;
    }

    uint GetInstanceInclusionMaskTranslucent()
    {
        uint FirstScatterType = GetFirstScatterType();
        if (FirstScatterType == PATHTRACER_SCATTER_CAMERA ||
            FirstScatterType == PATHTRACER_SCATTER_REFRACT)
        {
            return PATHTRACER_MASK_CAMERA_TRANSLUCENT;
        }
        if (FirstScatterType == PATHTRACER_SCATTER_SPECULAR)
        {
            return PATHTRACER_MASK_REFLECTION_TRANSLUCENT;
        }
        return PATHTRACER_MASK_INDIRECT_TRANSLUCENT;
    }

	bool UpdateScatterType(uint ScatterType)
	{
		// Keep track of the first non-refractive scatter type, and leave it "locked" beyond this
		// This ensures a particular pixel can only contribute to one lighting component.
		// The reason to not split refractive events is that you typically want to treat refracted paths the same as camera paths.
		// For example, when extracting just the diffuse component of a character wearing glasses, you would want the directly visible diffuse
		// and refracted diffuse to be treated the same.
		if (!HasMadeContributionScatter())
		{
			SetFirstScatterType(ScatterType);
		}
		switch (GetFirstScatterType())
		{
			// In these cases, only keep tracing if the corresponding lighting component is enabled
			case PATHTRACER_SCATTER_DIFFUSE:  return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_DIFFUSE ) != 0;
			case PATHTRACER_SCATTER_SPECULAR: return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_SPECULAR) != 0;
			case PATHTRACER_SCATTER_VOLUME:   return (EnabledIndirectLightingContributions & PATHTRACER_CONTRIBUTION_VOLUME  ) != 0;
		}
		// In all other cases, keep tracing, we haven't made a final decision on what this path is yet
		return true;
	}

	bool ShouldAccumulateEmissive()
	{
		uint EnabledLightingContributions = HasMadeContributionScatter() ? EnabledIndirectLightingContributions : EnabledDirectLightingContributions;
		return (EnabledLightingContributions & PATHTRACER_CONTRIBUTION_EMISSIVE) != 0;
	}

	// The following functions allow us to scale down the contributions for lighting components before we have scattered
	bool ShouldAccumulateDiffuse()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_DIFFUSE) != 0);
	}

	bool ShouldAccumulateSpecular()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_SPECULAR) != 0);
	}

	bool ShouldAccumulateVolume()
	{
		return (HasMadeContributionScatter() || (EnabledDirectLightingContributions & PATHTRACER_CONTRIBUTION_VOLUME) != 0);
	}

	float2 GetDiffuseSpecularScale(bool bIsVolumeSample)
	{
		float2 Result = 0.0;
		if (bIsVolumeSample)
		{
			if (ShouldAccumulateVolume())
			{
				Result.x = 1.0;
			}
		}
		else
		{
			if (ShouldAccumulateDiffuse())
			{
				Result.x = 1.0;
			}
			if (ShouldAccumulateSpecular())
			{
				Result.y = 1.0;
			}
		}
		return Result;
	}

	void WritePixel(uint2 TextureIndex)
	{
		// Avoid reading the old pixel on the first sample on the off-chance there is a NaN/Inf pixel ...
		float4 OldPixel    = SampleIndex > 0 ? RWSceneColor[TextureIndex] : 0;
		float4 OldDiffuse  = SampleIndex > 0 ? RWSceneDiffuseColor[TextureIndex]   : 0;
		float4 OldSpecular = SampleIndex > 0 ? RWSceneSpecularColor[TextureIndex]   : 0;
		float4 OldNormal   = SampleIndex > 0 ? RWSceneNormal[TextureIndex]   : 0;
		float OldRoughness = SampleIndex > 0 ? RWSceneRoughness[TextureIndex] : 0;
#if CLAMP_PATH_INTENSITY
        // Clamping is done with PreExposure folded in so that clamping is relative to current exposure level
		const float4 PixelValue = float4(Radiance, Alpha);
#else
        const float4 PixelValue = float4(Radiance * View.PreExposure, Alpha);
#endif

		const float3 Diffuse = GetDiffuseColor();
		const float3 Specular = GetSpecularColor();
		const float3 Normal = GetNormal();
		const float Roughness = GetRoughness();

		RWSceneColor[TextureIndex] = lerp(OldPixel, PixelValue, SampleBlendFactor);
		RWSceneDiffuseColor[TextureIndex].xyz = lerp(OldDiffuse.xyz, Diffuse, SampleBlendFactor);
		RWSceneSpecularColor[TextureIndex].xyz = lerp(OldSpecular.xyz, Specular, SampleBlendFactor);
		RWSceneNormal[TextureIndex] = float4(lerp(OldNormal, float4(Normal.xyz, Roughness), SampleBlendFactor));
		RWSceneRoughness[TextureIndex] = lerp(OldRoughness, Roughness, SampleBlendFactor);
	}

	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	void WriteDepth(uint2 TextureIndex)
	{
		float OldLinearDepth = SampleIndex > 0 ? RWSceneLinearDepth[TextureIndex] : FLOAT_MAX;
		float OldDepth = SampleIndex > 0 ? RWSceneDepth[TextureIndex] : 0;
		RWSceneLinearDepth[TextureIndex] = min(OldLinearDepth, DepthZ);
		RWSceneDepth[TextureIndex] = max(OldDepth, ConvertToDeviceZ(DepthZ));
	}
	#endif

    void AccumulateRadiance(float3 PathRadiance)
    {
#if CLAMP_PATH_INTENSITY
        PathRadiance *= View.PreExposure;
    #if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
        // camera rays, clamp at a fairly high level for half-float final storage into SceneColor buffer
        float MaxIntensity = MaxHalfFloat;
    #else
        // User asked for path contributions to be clamped to reduce fireflies from indirect paths
        // This puts an upper bound on variance within the pixel at the expense of bias
        float MaxIntensity = MaxPathIntensity;
    #endif
        // compute factor that will clamp the intensity but preserve the color
        float Adjust = MaxIntensity / max(max(MaxIntensity, PathRadiance.x), max(PathRadiance.y, PathRadiance.z));
        // accumulate into the total
        Radiance += PathRadiance * Adjust;
#else
        Radiance += PathRadiance;
#endif
    }

};




float3 GetLightingChannelMaskDebugColor(int LightingChannelMask)
{
	float r = LightingChannelMask & 1 ? 1.0 : 0.1;
	float g = LightingChannelMask & 2 ? 1.0 : 0.1;
	float b = LightingChannelMask & 4 ? 1.0 : 0.1;
	return float3(r, g, b);
}

float3 AnisoDebugColor(float InAniso)
{
	// Follow same mapping than raster buffer visualization (BP located in Engine/Contents/Anisotropy)
	float AniG = saturate( InAniso);
	float AniB = saturate(-InAniso);
	return float3(0.0, pow(AniG, 2.2), pow(AniB, 2.2));
}

#if PATH_TRACER_PRIMARY_RAYS

FPackedPathTracingPayload TraceTransparentRay(inout FPathState PathState, int Bounce)
{
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	const uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES | GetPathTracingHairRayFlags(PathState.Ray.Origin);
#else
	const uint RayFlags = GetPathTracingHairRayFlags(PathState.Ray.Origin);
#endif
	const uint MissShaderIndex = 0;
    const uint InstanceInclusionMaskOpaque = PathState.GetInstanceInclusionMaskOpaque();
	const float2 RandSample = PathState.RandSequence.Get2D();

	FRISContext HitSample = InitRISContext(RandSample.x);

	float3 ResultThroughput = 0;
    FPackedPathTracingPayload ResultPayload; ResultPayload.SetMiss();
	FPackedPathTracingPayload OpaquePackedPayload = InitPathTracingPayload(PathState.GetFirstScatterType(), PathState.GetPathRoughness());
	OpaquePackedPayload.SetDBufferA(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetDBufferB(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetDBufferC(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetStochasticSlabRand(RandSample.y);
	OpaquePackedPayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT);

#if PATH_TRACER_USE_SER 
	{
		NvHitObject Hit;
		NvTraceRayHitObject(
			TLAS,
			RayFlags,
			InstanceInclusionMaskOpaque,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			PathState.Ray.GetNativeDesc(),
			OpaquePackedPayload,
			Hit);
	#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
		NvReorderThread(Hit);
	#endif
		NvInvokeHitObject(TLAS, Hit, OpaquePackedPayload);
	}
#else
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMaskOpaque,
		RAY_TRACING_SHADER_SLOT_MATERIAL,
		RAY_TRACING_NUM_SHADER_SLOTS,
		MissShaderIndex,
		PathState.Ray.GetNativeDesc(),
		OpaquePackedPayload);
#endif
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
    if (OpaquePackedPayload.IsDecalReceiver() && max(MeshDecalRoughnessCutoff, DecalRoughnessCutoff) > 0.0)
    {
        FDecalShaderPayload DecalPayload = InitDecalShaderPayload(PathState.Ray.Origin + OpaquePackedPayload.HitT * PathState.Ray.Direction);
        // Evaluate mesh decals and decal actors
        EvaluateDecals(
            PathState.Ray.Direction,
            PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_CAMERA_TRANSLUCENT,
            OpaquePackedPayload.HitT,
            0.0,
            DecalPayload
        );

        // Handle d-buffer requests (will potentially re-trace the primary ray if needed)
        PATH_TRACER_DECAL_PROCESS_DBUFFER(OpaquePackedPayload, 1.0, RandSample.y)

        // Now that we know the real decal data, apply it
        ApplyDecal(DecalPayload, OpaquePackedPayload, 0.0);
    }
#endif

	if (OpaquePackedPayload.IsHit())
	{
		// TODO: If hit, also process decals, etc ...

		// Now that we know where the opaque hit is, trace the remaining portion of the ray prior to this
		PathState.Ray.TMax = asfloat(asuint(OpaquePackedPayload.HitT) - 1);
	}

	for (;;)
	{
#if PATH_TRACER_INCLUDE_TRANSLUCENT
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(PathState.GetFirstScatterType(), PathState.GetPathRoughness());
		PackedPayload.SetDBufferA(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferB(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferC(float4(0, 0, 0, 1));
		PackedPayload.SetStochasticSlabRand(RandSample.y);
		if (OpaquePackedPayload.IsHit())
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = PathState.Ray.Origin + OpaquePackedPayload.HitT * PathState.Ray.Direction;
			PackedPayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH);
			PackedPayload.SetSceneDepth(dot(TranslatedWorldPos, ViewZ));
		}

        const uint InstanceInclusionMaskTranslucent = PathState.GetInstanceInclusionMaskTranslucent();
#if PATH_TRACER_USE_SER 
		{
			NvHitObject Hit;
			NvTraceRayHitObject(
				TLAS,
				RayFlags,
				InstanceInclusionMaskTranslucent,
				RAY_TRACING_SHADER_SLOT_MATERIAL,
				RAY_TRACING_NUM_SHADER_SLOTS,
				MissShaderIndex,
				PathState.Ray.GetNativeDesc(),
				PackedPayload,
				Hit);
		#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
			NvReorderThread(Hit);
		#endif
			NvInvokeHitObject(TLAS, Hit, PackedPayload);
		}
#else
		// Trace the ray
		TraceRay(
			TLAS,
			RayFlags,
			InstanceInclusionMaskTranslucent,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			PathState.Ray.GetNativeDesc(),
			PackedPayload);
#endif
		if (PackedPayload.IsMiss())
		{
			// If we miss the translucent geometry, the next hit is the opaque one
			PackedPayload = OpaquePackedPayload;
		}

		// Loop over lights to capture their contribution
		// #dxr_todo: if we have lots of lights, having some hierarchical structure would be better ....
		// number of directly visible lights for the first bounce
#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		// Run this for camera rays only, after the first bounce, we don't need to include lights in the trace call
		// because NEE handled it for us
		for (uint LightId = 0, NumVisibleLights = SceneVisibleLightCount; LightId < NumVisibleLights; ++LightId)
		{
			FRayDesc LightRay = PathState.Ray;
			LightRay.TMax = PackedPayload.IsMiss() ? PathState.Ray.TMax : PackedPayload.HitT;
			FLightHit LightHit = TraceLight(LightRay, LightId);
			if (LightHit.IsHit())
			{
				// create a virtual surface hit so we only need to worry about volume transmission in one place
				// NOTE: returning only a single light hit here causes lights at infinity to occlude each other
				//       this is most likely what artists would want (imagine a scene with two suns for example)
				//       but is not consistent with how reflections behave and the sorting order will be arbitrary
				FPathTracingPayload LightHitPayload = (FPathTracingPayload)0; // clear all fields
				LightHitPayload.HitT = LightHit.HitT;
				LightHitPayload.Radiance = LightHit.Radiance * SceneLights[LightId].CameraColorScale;
				LightHitPayload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
				// light hits are fully transparent to match how the light loop works (the light surface does not cast a shadow)
				LightHitPayload.BSDFOpacity = 1.0;
				//LightHitPayload.TransparencyColor = LightHit.HitT == RAY_DEFAULT_T_MAX ? 1.0 - BackgroundAlpha : 1.0;
				LightHitPayload.TransparencyColor = 1;
				LightHitPayload.SetFrontFace();
				PackedPayload = PackPathTracingPayload(LightHitPayload);
			}
		}
#endif

#else // PATH_TRACER_INCLUDE_TRANSLUCENT

#define PackedPayload OpaquePackedPayload

#endif // PATH_TRACER_INCLUDE_TRANSLUCENT

		// If miss - exit
		if (PackedPayload.IsMiss())
		{
			// still no hit -- the ray must have escaped to the background, we are done
			// TODO: could add background Alpha here
			break;
		}

	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		if (!PackedPayload.IsHoldout())
		{
			PathState.Alpha += Luminance(PathState.PathThroughput * (1.0 - PackedPayload.UnpackTransparencyColor()));
		}
	#endif

		// Apply Beer's law
		float3 SigmaT = PathState.GetSigmaT();
		PathState.PathThroughput *= select(SigmaT > 0.0, exp(-SigmaT * (PackedPayload.HitT - PathState.Ray.TMin)), 1.0);

		if (PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA || EnableEmissive)
		{
            PathState.AccumulateRadiance(PathState.PathThroughput * PackedPayload.UnpackRadiance());
		}

	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
		if (PackedPayload.ShouldOutputDepth() && PathState.DepthZ == 0.0)
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = PathState.Ray.Origin + PackedPayload.HitT * PathState.Ray.Direction;
			PathState.DepthZ = dot(TranslatedWorldPos, ViewZ);
			PathState.SetNormal(PackedPayload.UnpackWorldNormal());
		}
	#endif

		if (Bounce < MaxBounces)
		{
			FPathTracingPayload HitPayload = UnpackPathTracingPayload(PackedPayload, PathState.Ray);
			AdjustPayloadAfterUnpack(HitPayload, true);
			float3 Contrib = PathState.PathThroughput * EstimateMaterialAlbedo(HitPayload);

			// only allow albedo tracking through a very narrow range of roughness values
			const float DenoiserMinRoughness = 0.000;
			const float DenoiserMaxRoughness = 0.004;
			float DenoiserContrib = 1 - saturate((PathState.GetPathRoughness() - DenoiserMinRoughness) / (DenoiserMaxRoughness - DenoiserMinRoughness));
			DenoiserContrib *= HitPayload.BSDFOpacity;
			// TODO: track denoiser signal through reflections/refractions
			if (DenoiserContrib > 0 && PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA)
			{
				float3 DenoiserThroughput = PathState.PathThroughput * DenoiserContrib;

				// TEMP: If the material has a valid SSS lobe, just merge it into diffuse for now
				FSSSRandomWalkInfo SSS = GetMaterialSSSInfo(HitPayload, PathState.Ray.Direction);
				if (any(SSS.Weight > 0))
				{
					HitPayload.DiffuseColor += SSS.Color;
				}

				// Accumulate G-Buffer like data for the denoiser
				float3 RoughnessData = HitPayload.RoughnessData;
				float Roughness = lerp(RoughnessData.x, RoughnessData.y, RoughnessData.z); // mix both lobes to get an average value
				DenoiserThroughput *= HitPayload.WeightV * lerp(1.0, HitPayload.TransmittanceN, HitPayload.CoverageAboveAlongN);

				float NormThroughput = Luminance(DenoiserThroughput);
				float3 DiffuseColor = HitPayload.DiffuseColor;
				float3 SpecularColor = HitPayload.SpecularColor;

				float MinRoughness = max(DenoiserMinRoughness, PathState.GetPathRoughness());
				float DenoiserRoughnessWeight = saturate((Roughness - MinRoughness) / (DenoiserMaxRoughness - MinRoughness));
#if 1
				// Remapping of NoV seems to be required by DLSS
				// TODO: Could use the energy conservation routines and tweak diffuse as well
				float NoV = saturate(dot(-PathState.Ray.Direction, HitPayload.WorldNormal));
				float x = NoV * NoV;
				NoV = 0.75 * x / (x + (1 - NoV) * (1 - NoV));
				float3 F = EnvBRDF(HitPayload.SpecularColor, Roughness, NoV);
#else
				float3 F = HitPayload.SpecularColor;
#endif

				// Accumulate contributions to denoiser channels
				float3 AccumDiffColor = PathState.GetDiffuseColor();
				float3 AccumSpecColor = PathState.GetSpecularColor();
				float AccumRoughness = PathState.GetRoughness();
				AccumDiffColor += DenoiserThroughput * HitPayload.DiffuseColor;
				AccumSpecColor += DenoiserThroughput * F * DenoiserRoughnessWeight;
				AccumRoughness += NormThroughput * Roughness;
				
				PathState.SetDiffuseSpecularColor(AccumDiffColor, AccumSpecColor);
				PathState.SetRoughness(AccumRoughness);
			}

			float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
			if (HitSample.Accept(SelectionWeight))
			{
				// stash this hit as a candidate return value
				ResultPayload = PackedPayload;
				ResultThroughput = PathState.PathThroughput / SelectionWeight;
			}
		}

		// account for local transparency change
		PathState.PathThroughput *= PackedPayload.UnpackTransparencyColor();

		// prepare next step around the loop (if any throughput is left)
		// retrace the exact same ray with TMin one ulp past the hit we just found
		PathState.Ray.TMin = ComputeNewTMin(PathState.Ray.Origin, PathState.Ray.Direction, PackedPayload.HitT);
		if (!(PathState.Ray.TMin < PathState.Ray.TMax) || all(PathState.PathThroughput == 0))
		{
			break;
		}
	}
	PathState.PathThroughput = ResultThroughput * HitSample.GetNormalization();
	return ResultPayload;
}

float LightContribToSelectionWeight(float3 C)
{
#if 0
    return C.x + C.y + C.z;
#elif 1
    // Seems to behave best overall
    return Luminance(C);
#elif 0
    return dot(C, float3(0.25, 0.5, 0.25));
#elif 0
    return max3(C.x, C.y, C.z);
#else
    return 1;
#endif
}

bool PathTracingDebugCore(inout FPathState PathState, int Bounce)
{
	FPackedPathTracingPayload PackedPayload = TraceTransparentRay(PathState, Bounce);

	if (PackedPayload.IsMiss())
	{
		// We didn't hit anything, done
		return false;
	}

	const float3 TranslatedWorldPos = PathState.Ray.Origin + PackedPayload.HitT * PathState.Ray.Direction;
	const float3 WorldNormal = PackedPayload.UnpackWorldNormal();
	const uint PrimitiveLightingChannelMask = PackedPayload.GetPrimitiveLightingChannelMask();

	const bool bLastBounce = Bounce == MaxBounces;

	FLightLoopCount LightLoopCount = LightGridLookup(TranslatedWorldPos);

	// Compute direct lighting
	FPathTracingPayload Payload = UnpackPathTracingPayload(PackedPayload, PathState.Ray);
	AdjustPayloadAfterUnpack(Payload, true);
	// TEMP: If the material has a valid SSS lobe, just merge it into diffuse for now
	FSSSRandomWalkInfo SSS = GetMaterialSSSInfo(Payload, PathState.Ray.Direction);
	if (any(SSS.Weight > 0))
	{
		Payload.DiffuseColor += SSS.Color;
	}

	// Generate main random number for direct+indirect sampling
	float4 RandSample = PathState.RandSequence.Get4D();

#if EVALUATE_MATERIAL_BSDF
	// Sample real material
	FMaterialSample MaterialSample = SampleMaterial(-PathState.Ray.Direction, Payload, RandSample.xyz);
#else
    // Hard-coded diffuse lobe for benchmarking
	const float4 SampledValue = CosineSampleHemisphereWorld(RandSample.xy, Payload.WorldNormal);
    FMaterialSample MaterialSample = CreateMaterialSample(SampledValue.xyz, Payload.BSDFOpacity * Payload.DiffuseColor, SampledValue.w, 1.0, 1.0, PATHTRACER_SCATTER_DIFFUSE);
#endif

#if LIGHT_LOOP_METHOD == 0

    #if LIGHT_SELECTION_METHOD == 0
    float LightPickPdf = rcp(LightLoopCount.NumLights);
    {
        int LightId = GetLightId(min(int(RandSample.z * LightLoopCount.NumLights), LightLoopCount.NumLights - 1), LightLoopCount);
    #else
        #if LIGHT_SELECTION_METHOD == 1
         // Store CDF in an array, then sample from it (maintains stratification)
	float LightPickingCdf[RAY_TRACING_LIGHT_COUNT_MAXIMUM];
    float LightPickingCdfSum = 0;
        #elif LIGHT_SELECTION_METHOD == 2
        // Use RIS to pick a light in a single pass (worse stratifiication, but less storage)
    FRISContext LightPicker = InitRISContext(RandSample.z);
    int SampledLightId = 0;
    float LightPickPdf = 0;
        #endif

	for (uint Index = 0, Num = LightLoopCount.NumLights; Index < Num; ++Index)
	{
		uint LightId = GetLightId(Index, LightLoopCount);
		float LightEstimate = EstimateLight(LightId, TranslatedWorldPos, WorldNormal, PrimitiveLightingChannelMask, false);
        LightEstimate *= PathState.HasMadeDiffuseOrSpecularScatter() ? GetLightIndirectScale(LightId, PathState.GetPathRoughness()) : 1.0;
        #if LIGHT_SELECTION_METHOD == 1
		LightPickingCdfSum += LightEstimate;
		LightPickingCdf[Index] = LightPickingCdfSum;
        #elif LIGHT_SELECTION_METHOD == 2
        if (LightPicker.Accept(LightEstimate))
        {
            // Store this index as our chosen sample (so far)
            SampledLightId = LightId;
            LightPickPdf = LightEstimate;
        }
        #endif
	}

        #if LIGHT_SELECTION_METHOD == 1
	if (LightPickingCdfSum > 0)
	{
		// init worked
        int SampledIndex;
		float LightPickPdf = 0;

		SelectLight(RandSample.z * LightPickingCdfSum, LightLoopCount.NumLights, LightPickingCdf, SampledIndex, LightPickPdf);
		LightPickPdf /= LightPickingCdfSum;
        int LightId = GetLightId(SampledIndex, LightLoopCount);
        #elif LIGHT_SELECTION_METHOD == 2
    if (LightPicker.HasSample())
    {
        LightPickPdf /= LightPicker.GetNormalization();
        int LightId = SampledLightId;
        #endif // LIGHT_SELECTION_METHOD
    #endif // LIGHT_SELECTION_METHOD
		float2 LightRandSample = RandSample.xy;
		
        // Generate a sample on the light source
        FLightSample LightSample = SampleLight(LightId, LightRandSample, TranslatedWorldPos, WorldNormal);
        if (LightSample.Pdf > 0)
        {
            LightSample.RadianceOverPdf /= LightPickPdf;
            LightSample.Pdf *= LightPickPdf;

            float2 LightDiffuseSpecularScale = float2(GetLightDiffuseScale(LightId), GetLightSpecularScale(LightId));
    #if EVALUATE_MATERIAL_BSDF
            // Evaluate real material
            FMaterialEval MaterialEval = EvalMaterial(-PathState.Ray.Direction, LightSample.Direction, Payload, LightDiffuseSpecularScale);
    #else
            // Hard-coded diffuse lobe for benchmarking
            FMaterialEval MaterialEval = CreateMaterialEval(Payload.DiffuseColor * LightDiffuseSpecularScale.x, saturate(dot(WorldNormal, LightSample.Direction)) / PI);
    #endif
            float3 LightContrib = PathState.PathThroughput * LightSample.RadianceOverPdf * MaterialEval.Weight * MaterialEval.Pdf;
            LightContrib *= PathState.HasMadeDiffuseOrSpecularScatter() ? GetLightIndirectScale(LightId, PathState.GetPathRoughness()) : 1.0;

    #if USE_BSDF_SAMPLING
            LightContrib *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
    #endif
            // Trace shadow ray immediately
            FRayDesc ShadowRay;
            ShadowRay.Origin = TranslatedWorldPos;
            ShadowRay.Direction = LightSample.Direction;
            ShadowRay.TMin = 0;
            ShadowRay.TMax = LightSample.Distance;
            const float PositionBiasSign = Payload.IsMaterialTransmissive() ? sign(dot(Payload.WorldGeoNormal, ShadowRay.Direction)) : 1.0;
            const bool bCastShadows = CastsShadow(LightId);
            const uint MissShaderIndex = GetLightMissShaderIndex(LightId);
            ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
            LightContrib *= TraceShadowRay(ShadowRay, GetAverageRoughness(Payload), PathState.GetPathRoughness(), MissShaderIndex, bCastShadows);
            PathState.AccumulateRadiance(LightContrib);
        }
    } // LightPickingCdfSum > 0
    #if USE_BSDF_SAMPLING == 1

        #if BSDF_SAMPLING_METHOD == 1
    FRISContext BSDFSamplePicker = InitRISContext(RandSample.w);
    float SampledTMax = 0;
    float3 SampledLightContrib = 0;
    int SampledLightId = 0;
        #endif

        #if LIGHT_SELECTION_METHOD == 1
    float PreviousCdfValue = 0.0;
        #endif
    for (uint Index = 0, Num = LightLoopCount.NumMISLights; Index < Num; ++Index)
    {
        const uint LightId = GetLightId(Index, LightLoopCount);
        #if LIGHT_SELECTION_METHOD == 1
        const float CdfValue = LightPickingCdf[Index];
        const float LightPickPdf = (CdfValue - PreviousCdfValue) / LightPickingCdfSum;
        PreviousCdfValue = CdfValue;
        #endif
        FRayDesc ShadowRay;
        ShadowRay.Origin = TranslatedWorldPos;
        ShadowRay.Direction = MaterialSample.Direction;
        ShadowRay.TMin = 0;
        ShadowRay.TMax = RAY_DEFAULT_T_MAX;

        FLightHit LightResult = TraceLight(ShadowRay, LightId);
        if (LightResult.IsHit())
        {
            ShadowRay.TMax = LightResult.HitT;
            float3 LightContrib = PathState.PathThroughput * MaterialSample.Weight * LightResult.Radiance;
            LightContrib *= MaterialSample.ScatterType == PATHTRACER_SCATTER_DIFFUSE ? GetLightDiffuseScale(LightId) : GetLightSpecularScale(LightId);
            float IndirectScale = PathState.HasMadeDiffuseOrSpecularScatter() ? GetLightIndirectScale(LightId, PathState.GetPathRoughness()) : 1.0;
            LightContrib *= IndirectScale;

            #if LIGHT_SELECTION_METHOD == 2
            // We didn't store the light estimate in an array, so need to redo it now to get the probability of having picked this light
            float LightPickPdf = 0;
            if (LightPicker.HasSample())
            {
                float LightEstimate = EstimateLight(LightId, TranslatedWorldPos, WorldNormal, PrimitiveLightingChannelMask, false);
                LightEstimate *= IndirectScale;
                LightPickPdf = LightEstimate / LightPicker.GetNormalization();
            }
            #endif
            LightContrib *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf * LightPickPdf);
            #if BSDF_SAMPLING_METHOD == 0
            const bool bCastShadows = CastsShadow(LightId);
            const uint MissShaderIndex = GetLightMissShaderIndex(LightId);
            ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, MaterialSample.PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
            LightContrib *= TraceShadowRay(ShadowRay, MaterialSample.Roughness, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows);
            PathState.AccumulateRadiance(LightContrib);
            #elif BSDF_SAMPLING_METHOD == 1
            float SelectionWeight = LightContribToSelectionWeight(LightContrib);
            if (BSDFSamplePicker.Accept(SelectionWeight))
            {
                SampledLightId = LightId;
                SampledLightContrib = LightContrib / SelectionWeight;
                SampledTMax = ShadowRay.TMax;
            }
            #endif
        }
    }
        #if BSDF_SAMPLING_METHOD == 1
    if (BSDFSamplePicker.HasSample())
    {
        SampledLightContrib *= BSDFSamplePicker.GetNormalization();
        const bool bCastShadows = CastsShadow(SampledLightId);
        const uint MissShaderIndex = GetLightMissShaderIndex(SampledLightId);
        FRayDesc ShadowRay;
        ShadowRay.Origin = TranslatedWorldPos;
        ShadowRay.Direction = MaterialSample.Direction;
        ShadowRay.TMin = 0;
        ShadowRay.TMax = SampledTMax;
        ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, MaterialSample.PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
        SampledLightContrib *= TraceShadowRay(ShadowRay, MaterialSample.Roughness, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows);
        PathState.AccumulateRadiance(SampledLightContrib);
    }
        #endif
    #endif // USE_BSDF_SAMPLING == 1

#else // LIGHT_LOOP_METHOD != 0
    float2 LightRandSample = RandSample.xy;

    #if LIGHT_LOOP_METHOD == 2
    FRISContext TraceSample = InitRISContext(RandSample.z);
    float4 ShadowRayDirLength = 0;
	float3 ShadowRayContrib = 0;
	uint ShadowRayLightId = 0;
    #endif

    for (uint Index = 0, Num = LightLoopCount.NumLights; Index < Num; ++Index)
	{
        uint LightId = GetLightId(Index, LightLoopCount);
        const bool bCastShadows = CastsShadow(LightId);
        const uint MissShaderIndex = GetLightMissShaderIndex(LightId);
        float IndirectScale = PathState.HasMadeDiffuseOrSpecularScatter() ? GetLightIndirectScale(LightId, PathState.GetPathRoughness()) : 1.0;
        // Generate a sample on the light source
        FLightSample LightSample = SampleLight(LightId, LightRandSample, TranslatedWorldPos, WorldNormal);
        if (LightSample.Pdf > 0)
        {
            float2 LightDiffuseSpecularScale = float2(GetLightDiffuseScale(LightId), GetLightSpecularScale(LightId));

    #if EVALUATE_MATERIAL_BSDF
            // Evaluate real material
            FMaterialEval MaterialEval = EvalMaterial(-PathState.Ray.Direction, LightSample.Direction, Payload, LightDiffuseSpecularScale);
    #else
            // Hard-coded diffuse lobe for benchmarking
            FMaterialEval MaterialEval = CreateMaterialEval(Payload.DiffuseColor * LightDiffuseSpecularScale.x, saturate(dot(WorldNormal, LightSample.Direction)) / PI);
    #endif
            float3 LightContrib = PathState.PathThroughput * LightSample.RadianceOverPdf * MaterialEval.Weight * MaterialEval.Pdf;
            LightContrib *= IndirectScale;

    #if USE_BSDF_SAMPLING
            LightContrib *= MISWeightPower(LightSample.Pdf, MaterialEval.Pdf);
    #endif

    #if LIGHT_LOOP_METHOD == 2
        #if ACCUMULATE_UNSHADOWED_LIGHTS
            if (bCastShadows == false && MissShaderIndex == 0)
            {
                PathState.AccumulateRadiance(LightContrib);
            }
            else
        #endif
        #if FORCE_TRACE_INFINITE_LIGHTS
            if (Index < SceneInfiniteLightCount)
            {
                // Trace shadow ray immediately
                FRayDesc ShadowRay;
                ShadowRay.Origin = TranslatedWorldPos;
                ShadowRay.Direction = LightSample.Direction;
                ShadowRay.TMin = 0;
                ShadowRay.TMax = LightSample.Distance;
                const float PositionBiasSign = Payload.IsMaterialTransmissive() ? sign(dot(Payload.WorldGeoNormal, ShadowRay.Direction)) : 1.0;
                ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
                PathState.AccumulateRadiance(LightContrib * TraceShadowRay(ShadowRay, GetAverageRoughness(Payload), PathState.GetPathRoughness(), MissShaderIndex, bCastShadows));
            }
            else
        #endif
            {
                float SelectionWeight = LightContribToSelectionWeight(LightContrib);
                if (TraceSample.Accept(SelectionWeight))
                {
                    ShadowRayDirLength = float4(LightSample.Direction, LightSample.Distance);
                    ShadowRayContrib = LightContrib / SelectionWeight;
                    ShadowRayLightId = LightId;
                }
            }
    #else // LIGHT_LOOP_METHOD != 2
            // Trace shadow ray immediately
            FRayDesc ShadowRay;
            ShadowRay.Origin = TranslatedWorldPos;
            ShadowRay.Direction = LightSample.Direction;
            ShadowRay.TMin = 0;
            ShadowRay.TMax = LightSample.Distance;
            const float PositionBiasSign = Payload.IsMaterialTransmissive() ? sign(dot(Payload.WorldGeoNormal, ShadowRay.Direction)) : 1.0;
            ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, PositionBiasSign * Payload.WorldGeoNormal);
            PathState.AccumulateRadiance(LightContrib * TraceShadowRay(ShadowRay, GetAverageRoughness(Payload), PathState.GetPathRoughness(), MissShaderIndex, bCastShadows));
    #endif // LIGHT_LOOP_METHOD != 2
        } // LightSample.Pdf > 0

    #if USE_BSDF_SAMPLING
        // Generate a sample by intersecting a BSDF ray with the light
		{
            FRayDesc ShadowRay;
			ShadowRay.Origin = TranslatedWorldPos;
			ShadowRay.Direction = MaterialSample.Direction;
			ShadowRay.TMin = 0;
			ShadowRay.TMax = RAY_DEFAULT_T_MAX;

			FLightHit LightResult = TraceLight(ShadowRay, LightId);
			if (LightResult.IsHit())
			{
				ShadowRay.TMax = LightResult.HitT;
				float3 LightContrib = PathState.PathThroughput * MaterialSample.Weight * LightResult.Radiance;
				LightContrib *= MaterialSample.ScatterType == PATHTRACER_SCATTER_DIFFUSE ? GetLightDiffuseScale(LightId) : GetLightSpecularScale(LightId);
                LightContrib *= IndirectScale;
				LightContrib *= MISWeightPower(MaterialSample.Pdf, LightResult.Pdf);
        #if LIGHT_LOOP_METHOD == 2
            #if ACCUMULATE_UNSHADOWED_LIGHTS
                if (bCastShadows == false && MissShaderIndex == 0)
                {
                    PathState.AccumulateRadiance(LightContrib);
                }
                else
            #endif
            #if  FORCE_TRACE_INFINITE_LIGHTS
                if (Index < SceneInfiniteLightCount)
                {
                    ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, MaterialSample.PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
                    PathState.AccumulateRadiance(LightContrib * TraceShadowRay(ShadowRay, GetAverageRoughness(Payload), PathState.GetPathRoughness(), MissShaderIndex, bCastShadows));
                }
                else
            #endif
                {
                    // Defer tracing shadow until later
                    float SelectionWeight = LightContribToSelectionWeight(LightContrib);
                    if (TraceSample.Accept(SelectionWeight))
                    {
                        ShadowRayDirLength = float4(MaterialSample.Direction, LightResult.HitT);
                        ShadowRayContrib = LightContrib / SelectionWeight;
                        ShadowRayLightId = LightId | 0x80000000u;
                    }
                }
        #else // LIGHT_LOOP_METHOD != 2
                ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, MaterialSample.PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
                PathState.AccumulateRadiance(LightContrib * TraceShadowRay(ShadowRay, MaterialSample.Roughness, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows));
        #endif
            } // if (LightResult.IsHit())
		}
    #endif // USE_BSDF_SAMPLING
    } // End light loop

#if LIGHT_LOOP_METHOD == 2
    // Light loop gave us a valid candidate sample, trace it now
    if (TraceSample.HasSample())
    {
        const bool bIsBSDFSample = (ShadowRayLightId & 0x80000000u) != 0;
        ShadowRayLightId &= ~0x80000000u;
        const bool bCastShadows = CastsShadow(ShadowRayLightId);
        const uint MissShaderIndex = GetLightMissShaderIndex(ShadowRayLightId);
        const float ShadowRayRoughness = bIsBSDFSample ? MaterialSample.Roughness : GetAverageRoughness(Payload);
        const float PositionBiasSign = bIsBSDFSample ? MaterialSample.PositionBiasSign : (Payload.IsMaterialTransmissive() ? sign(dot(Payload.WorldGeoNormal, ShadowRayDirLength.xyz)) : 1.0);
        FRayDesc ShadowRay;
        ShadowRay.Origin = TranslatedWorldPos;
        ShadowRay.Direction = ShadowRayDirLength.xyz;
        ShadowRay.TMin = 0;
        ShadowRay.TMax = ShadowRayDirLength.w;

        ApplyRayBias(ShadowRay.Origin, PackedPayload.HitT, PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());

        ShadowRayContrib *= TraceSample.GetNormalization();
        PathState.AccumulateRadiance(ShadowRayContrib * TraceShadowRay(ShadowRay, ShadowRayRoughness, PathState.GetPathRoughness(), MissShaderIndex, bCastShadows));
    }
#endif

#endif // LIGHT_LOOP_METHOD != 0

	// don't generate another ray on the last bounce or second to last bounce if emissive materials are disabled
	if (bLastBounce || (Bounce + 1 == MaxBounces && EnableEmissive == 0))
	{
		return false;
	}

	// sample new direction

	// BSDF Sampling
	float3 Throughput = MaterialSample.Weight;
	float3 SampledDirection = MaterialSample.Direction;
	if (!PathState.UpdateScatterType(MaterialSample.ScatterType))
	{
		return false;
	}
	// enlarge roughness based on the chosen lobe roughness
	float PreviousPathRoughness = PathState.GetPathRoughness();
	PathState.SetPathRoughness(max(PreviousPathRoughness, MaterialSample.Roughness));


	// update the current extinction if we are crossing a boundary on glass or water
	if (MaterialSample.PositionBiasSign < 0 && Payload.IsMaterialSolidGlass())
	{
		const float3 LocalSigmaT = Payload.GetExtinction();
		float3 SigmaT = PathState.GetSigmaT();
		if (Payload.IsFrontFace())
		{
			// entering
			SigmaT += LocalSigmaT;
		}
		else
		{
			// exiting
			SigmaT -= LocalSigmaT;
			SigmaT = max(SigmaT, 0);
		}
		PathState.SetSigmaT(SigmaT);
	}

	float3 PathThroughput = PathState.PathThroughput;
	float3 NextPathThroughput = PathThroughput * Throughput;
	// Russian roulette:
	//   The probability of keeping the path should be roughly proportional to the weight at the current shade point,
	//  but just using MaterialWeight would miss out on cases where the path throughput changes color (like in a cornell
	//  box when bouncing between walls of different colors). So use the ratio of the brightest color channel in the
	//  previous and next throughput.
	//   The second tweak is to add a sqrt() around the probability to soften the termination probability (paths will last
	//  a little longer). This allows paths to go a bit deeper than the naive heuristic while still allowing them to terminate
	//  early. This makes RR effective from the very first bounce without needing to delay it.
	float ContinuationProb = sqrt(saturate(max3(NextPathThroughput.x, NextPathThroughput.y, NextPathThroughput.z) / max3(PathThroughput.x, PathThroughput.y, PathThroughput.z)));
	if (ContinuationProb < 1)
	{
		// If there is some chance we should terminate the ray, draw an extra random value
		float RussianRouletteRand = RandSample.w;
		if (RussianRouletteRand >= ContinuationProb)
		{
			// stochastically terminate the path
			return false;
		}
		PathThroughput = NextPathThroughput / ContinuationProb;
	}
	else
	{
		PathThroughput = NextPathThroughput;
	}
	// Setup next ray
	PathState.PathThroughput = PathThroughput;
	PathState.Ray.Origin = TranslatedWorldPos;
	PathState.Ray.Direction = SampledDirection;
	PathState.Ray.TMin = 0;
	PathState.Ray.TMax = RAY_DEFAULT_T_MAX;
	ApplyRayBias(PathState.Ray.Origin, PackedPayload.HitT, MaterialSample.PositionBiasSign * PackedPayload.UnpackWorldGeoNormal());
	return true;
}

#else // PATH_TRACER_PRIMARY_RAYS

// for debugging and to visualize
float3 GetPathTracingShadingModelColor(uint PathTracingShadingModelID)
{
	switch (PathTracingShadingModelID)
	{
		case PATH_TRACING_SHADINGMODELID_BASIC: 		return float3(0.1f, 1.0f, 0.1f); // Green
		case PATH_TRACING_SHADINGMODELID_FOLIAGE: 		return float3(0.2f, 0.2f, 0.8f); // Cyan
		case PATH_TRACING_SHADINGMODELID_CLEARCOAT: 	return float3(0.1f, 0.4f, 0.4f); // Brown
		case PATH_TRACING_SHADINGMODELID_SOLID_GLASS: 	return float3(1.0f, 0.1f, 0.1f); // Red
		case PATH_TRACING_SHADINGMODELID_THIN_GLASS: 	return float3(1.0f, 0.8f, 0.3f);
		case PATH_TRACING_SHADINGMODELID_EYE: 			return float3(0.3f, 1.0f, 1.0f);
		case PATH_TRACING_SHADINGMODELID_HAIR: 			return float3(0.6f, 0.1f, 0.5f);
		case PATH_TRACING_SHADINGMODELID_MEDIUM:		return float3(0.6f, 0.4f, 0.1f); // Brown
		case PATH_TRACING_SHADINGMODELID_VOLUMETRIC:	return float3(0.2f, 0.6f, 0.5f); // Cyan
		case PATH_TRACING_SHADINGMODELID_LAMBERT: 		return float3(0.1f, 0.1f, 0.2f); // Dark Blue
		default: return float3(1.0f, 1.0f, 1.0f); // White
	}
}


bool DebugModeRequiresFakeShading()
{
    // Does this debug mode require a bit of "fake" shading to enhance readability?
    switch (DebugMode)
    {
        case PATH_TRACER_DEBUG_VIZ_SHADING_MODEL:
        case PATH_TRACER_DEBUG_VIZ_LIGHTING_CHANNEL_MASK:
        case PATH_TRACER_DEBUG_VIZ_WORLD_POSITION:
        case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_COUNT:
        case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_AXIS:
        case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_LIST_HASH:
        case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_SINGULAR:
        case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_COUNT:
        case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_AXIS:
        case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_LIST_HASH:
        case PATH_TRACER_DEBUG_VIZ_VOLUME_LIGHT_COUNT:
        case PATH_TRACER_DEBUG_VIZ_HITKIND:
        case PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COUNT:
        case PATH_TRACER_DEBUG_VIZ_BACKFACING_NORMAL:
        case PATH_TRACER_DEBUG_VIZ_TRIANGLE_ID:
        case PATH_TRACER_DEBUG_VIZ_INSTANCE_ID:
        case PATH_TRACER_DEBUG_VIZ_CLUSTER_ID:
        case PATH_TRACER_DEBUG_VIZ_DECAL_COVERAGE:
        case PATH_TRACER_DEBUG_VIZ_DECAL_COLOR:
        case PATH_TRACER_DEBUG_VIZ_DECAL_NORMAL:
        case PATH_TRACER_DEBUG_VIZ_DECAL_RMS:
        case PATH_TRACER_DEBUG_VIZ_DECAL_RADIANCE:
        case PATH_TRACER_DEBUG_VIZ_DECAL_RESPONSE_MASK:
        case PATH_TRACER_DEBUG_VIZ_DECAL_USE_DBUFFER:
            return true;
    }
    return false;
}

FPathTracingResult PathTracingDebugCore(FRayDesc Ray, uint2 LaunchIndex)
{
	FPathTracingResult Result = (FPathTracingResult) 0;
	const uint RayFlags = DebugMode == PATH_TRACER_DEBUG_VIZ_HITKIND ? 0 : RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
	const uint MissShaderIndex = 0;
	int TransparentHitCount = 0;

	const float RandSample = RandomSequenceCreate(uint3(LaunchIndex, View.StateFrameIndex), 0, 1).Get1D();


	float3 PathThroughput = 1;
	FPackedPathTracingPayload OpaquePackedPayload = InitPathTracingPayload(PATHTRACER_SCATTER_CAMERA, 0.0);
	OpaquePackedPayload.SetDBufferA(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetDBufferB(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetDBufferC(float4(0, 0, 0, 1));
	OpaquePackedPayload.SetStochasticSlabRand(RandSample);
    uint DebugFlag = 0;
    switch (DebugMode)
    {
        case PATH_TRACER_DEBUG_VIZ_TRIANGLE_ID: DebugFlag = PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_TRIANGLE_ID; break;
        case PATH_TRACER_DEBUG_VIZ_INSTANCE_ID: DebugFlag = PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_INSTANCE_ID; break;
        case PATH_TRACER_DEBUG_VIZ_CLUSTER_ID:  DebugFlag = PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_CLUSTER_ID; break;
    }
	OpaquePackedPayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT | DebugFlag);


	TraceRay(
		TLAS,
		RayFlags,
		PATHTRACER_MASK_CAMERA_OPAQUE,
		RAY_TRACING_SHADER_SLOT_MATERIAL,
		RAY_TRACING_NUM_SHADER_SLOTS,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		OpaquePackedPayload);

    FDecalShaderPayload DecalPayload = InitDecalShaderPayload(Ray.Origin + OpaquePackedPayload.HitT * Ray.Direction);
    switch (DebugMode)
    {
        // Only evaluate the decals if the visualization needs to actually show decal data
        case PATH_TRACER_DEBUG_VIZ_DECAL_COVERAGE:
        case PATH_TRACER_DEBUG_VIZ_DECAL_COLOR:
        case PATH_TRACER_DEBUG_VIZ_DECAL_NORMAL:
        case PATH_TRACER_DEBUG_VIZ_DECAL_RMS:
        case PATH_TRACER_DEBUG_VIZ_DECAL_RADIANCE:
        {
            if (OpaquePackedPayload.IsDecalReceiver() && max(MeshDecalRoughnessCutoff, DecalRoughnessCutoff) > 0.0)
            {
                // Evaluate mesh decals and decal actors
                EvaluateDecals(
                    Ray.Direction,
                    PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_CAMERA_TRANSLUCENT,
                    OpaquePackedPayload.HitT,
                    0.0,
                    DecalPayload
                );
            }
            break;
        }
    }

	if (OpaquePackedPayload.IsHit())
	{
		// TODO: If hit, also process decals, etc ...

		// Now that we know where the opaque hit is, trace the remaining portion of the ray prior to this
		Ray.TMax = asfloat(asuint(OpaquePackedPayload.HitT) - 1);
	}

	if (DebugMode == PATH_TRACER_DEBUG_VIZ_VOLUME_LIGHT_COUNT)
	{
		int NumLights = 0;
		for (int LightId = SceneInfiniteLightCount; LightId < SceneLightCount; LightId++)
		{
			FVolumeLightSampleSetup LightSetup = PrepareLightVolumeSample(LightId, Ray.Origin, Ray.Direction, Ray.TMin, Ray.TMax);
			if (LightSetup.IsValid())
			{
				float LightProb = LightSetup.LightImportance * GetVolumetricScatteringIntensity(LightId);
				if (LightProb > 0)
				{
					NumLights++;
				}
			}
		}

        Result.Radiance.rgb = 0.18;
        if (NumLights > 0)
        {
            Result.Radiance.rgb = (NumLights > VisualizeMaxCount) ? float3(1, 0.1, 1) : BlueGreenRedColorMap(saturate(float(NumLights) / VisualizeMaxCount));
        }

		if (OpaquePackedPayload.IsHit())
		{
			// add some fake shading to help with definition
			Result.Radiance.rgb *= abs(dot(OpaquePackedPayload.UnpackWorldNormal(), Ray.Direction));
		}
		return Result;
	}

	for (;;)
	{
#if PATH_TRACER_INCLUDE_TRANSLUCENT
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(PATHTRACER_SCATTER_CAMERA, 0.0);
		PackedPayload.SetDBufferA(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferB(float4(0, 0, 0, 1));
		PackedPayload.SetDBufferC(float4(0, 0, 0, 1));
		PackedPayload.SetStochasticSlabRand(RandSample);
        PackedPayload.SetFlag(DebugFlag);
		if (OpaquePackedPayload.IsHit())
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = Ray.Origin + OpaquePackedPayload.HitT * Ray.Direction;
			PackedPayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH);
			PackedPayload.SetSceneDepth(dot(TranslatedWorldPos, ViewZ));
		}

		// Trace the ray
		TraceRay(
			TLAS,
			RayFlags,
			PATHTRACER_MASK_CAMERA_TRANSLUCENT,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			Ray.GetNativeDesc(),
			PackedPayload);

		if (PackedPayload.IsMiss())
		{
			// If we miss the translucent geometry, the next hit is the opaque one
			PackedPayload = OpaquePackedPayload;
		}
		else
		{
			TransparentHitCount++;
		}
#else // PATH_TRACER_INCLUDE_TRANSLUCENT
		#define PackedPayload OpaquePackedPayload // just rename
#endif // PATH_TRACER_INCLUDE_TRANSLUCENT
		// If miss - exit
		if (PackedPayload.IsMiss())
		{
			// still no hit -- the ray must have escaped to the background, we are done
			// TODO: could add background Alpha here
			break;
		}

		if (!PackedPayload.IsHoldout())
		{
			Result.Radiance.a += Luminance(PathThroughput * (1.0 - PackedPayload.UnpackTransparencyColor()));
		}

		{
			FPathTracingPayload HitPayload = UnpackPathTracingPayload(PackedPayload, Ray);
			AdjustPayloadAfterUnpack(HitPayload, true);

			const float3 TranslatedWorldPos = Ray.Origin + PackedPayload.HitT * Ray.Direction;
			const float3 WorldNormal = HitPayload.WorldNormal;

			float3 Color = 0, ColorAdd = 0;
			float ExtraAlpha = 0;
			switch (DebugMode)
			{
				case PATH_TRACER_DEBUG_VIZ_RADIANCE: 					Color = HitPayload.Radiance * View.PreExposure; break;
				case PATH_TRACER_DEBUG_VIZ_WORLD_NORMAL: 				Color = WorldNormal * 0.5 + 0.5; break;
				case PATH_TRACER_DEBUG_VIZ_WORLD_SMOOTH_NORMAL:			Color = HitPayload.WorldSmoothNormal * 0.5 + 0.5; break;
				case PATH_TRACER_DEBUG_VIZ_WORLD_GEO_NORMAL:			Color = HitPayload.WorldGeoNormal * 0.5 + 0.5; break;
				case PATH_TRACER_DEBUG_VIZ_BASE_COLOR:					Color = lerp(HitPayload.DiffuseColor, HitPayload.SpecularColor, F0RGBToMetallic(HitPayload.SpecularColor)); break;
				case PATH_TRACER_DEBUG_VIZ_DIFFUSE_COLOR:				Color = HitPayload.DiffuseColor; break;
				case PATH_TRACER_DEBUG_VIZ_SPECULAR_COLOR:				Color = HitPayload.SpecularColor; break;
				case PATH_TRACER_DEBUG_VIZ_METALLIC:					Color = F0RGBToMetallic(HitPayload.SpecularColor); break;
				case PATH_TRACER_DEBUG_VIZ_SPECULAR:					Color = F0RGBToDielectricSpecular(HitPayload.SpecularColor); break;
				case PATH_TRACER_DEBUG_VIZ_ROUGHNESS:					Color = HitPayload.RoughnessData; break;
				case PATH_TRACER_DEBUG_VIZ_ANISOTROPY:					Color = AnisoDebugColor(HitPayload.Anisotropy); break;
				case PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COLOR:			Color = HitPayload.TransparencyColor; break;
				case PATH_TRACER_DEBUG_VIZ_IOR:							Color = HitPayload.Ior; break;
				case PATH_TRACER_DEBUG_VIZ_SHADING_MODEL:				Color = GetPathTracingShadingModelColor(HitPayload.PathTracingShadingModelID); break;
				case PATH_TRACER_DEBUG_VIZ_LIGHTING_CHANNEL_MASK:		Color = GetLightingChannelMaskDebugColor(HitPayload.PrimitiveLightingChannelMask); break;
				case PATH_TRACER_DEBUG_VIZ_WORLD_POSITION:
				{
					float3 AbsoluteWorldPosition = DFDemote(DFFastSubtract(TranslatedWorldPos, ResolvedView.PreViewTranslation));
					const float CellWidth = 10.0f; // 10cm
					int3 Coord = int3(round(AbsoluteWorldPosition / CellWidth));
					uint HashIndex = GridHash(int4(Coord, 0)).x;
					Color = HUEtoRGB((HashIndex & 0xFFFFFF) * 5.96046447754e-08);
					break;
				}
				case PATH_TRACER_DEBUG_VIZ_WORLD_TANGENT: 				Color = PackedPayload.UnpackWorldTangent() * 0.5 + 0.5; break;
				case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_COUNT:
				case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_AXIS:
                case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_LIST_HASH:
                case PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_SINGULAR:
				{
                    FLightLoopCount LightLoopCount = LightGridLookup(TranslatedWorldPos);
					Color = LightGridVisualize(LightLoopCount, DebugMode, VisualizeMaxCount);
					break;
				}
				case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_COUNT:
				case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_AXIS:
                case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_LIST_HASH:
				{
					FDecalLoopCount DecalLoopCount = DecalGridLookup(TranslatedWorldPos);
					Color = PackedPayload.IsDecalReceiver() ? DecalGridVisualize(DecalLoopCount, DebugMode, VisualizeMaxCount) : 0.18;
					break;
				}
				case PATH_TRACER_DEBUG_VIZ_HITKIND:
				{
					Color = (PackedPayload.IsFrontFace() ? float3(0, 1, 0) : float3(1, 0, 0));
					break;
				}
				case PATH_TRACER_DEBUG_VIZ_SSS_COLOR:
				case PATH_TRACER_DEBUG_VIZ_SSS_RADIUS:
				case PATH_TRACER_DEBUG_VIZ_SSS_WEIGHT:
				case PATH_TRACER_DEBUG_VIZ_SSS_PHASE:
				{
					FSSSRandomWalkInfo SSS = GetMaterialSSSInfo(HitPayload, Ray.Direction);
					if (any(SSS.Weight > 0))
					{
						if (DebugMode == PATH_TRACER_DEBUG_VIZ_SSS_COLOR)
						{
							Color = SSS.Color;
						}
						else if (DebugMode == PATH_TRACER_DEBUG_VIZ_SSS_RADIUS)
						{
							Color = SSS.Radius;
						}
						else if (DebugMode == PATH_TRACER_DEBUG_VIZ_SSS_WEIGHT)
						{
							Color = SSS.Weight;
						}
						else if (DebugMode == PATH_TRACER_DEBUG_VIZ_SSS_PHASE)
						{
							Color = BlueGreenRedColorMap(0.5 * SSS.G + 0.5);
						}
					}
					break;
				}
				case PATH_TRACER_DEBUG_VIZ_BSDF_OPACITY:    Color = HitPayload.BSDFOpacity; break;
				case PATH_TRACER_DEBUG_VIZ_FUZZ_COLOR:     	Color = HitPayload.HasFuzzLobe() ? HitPayload.FuzzColor     : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_FUZZ_ROUGHNESS:	Color = HitPayload.HasFuzzLobe() ? HitPayload.FuzzRoughness : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_FUZZ_AMOUNT:		Color = HitPayload.HasFuzzLobe() ? HitPayload.FuzzAmount    : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_SUBSTRATE_WEIGHT_V: 			Color = HitPayload.WeightV; 			break;
				case PATH_TRACER_DEBUG_VIZ_SUBSTRATE_TRANSMITTANCE_N: 	Color = HitPayload.TransmittanceN; 		break;
				case PATH_TRACER_DEBUG_VIZ_SUBSTRATE_COVERAGE_ABOVE_N: 	Color = HitPayload.CoverageAboveAlongN; break;
				case PATH_TRACER_DEBUG_VIZ_SUBSTRATE_F90:				Color = HitPayload.SpecularEdgeColor;   break;
				case PATH_TRACER_DEBUG_VIZ_EYE_IRIS_MASK:      Color = (HitPayload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE ) ?             HitPayload.GetEyeIrisMask()      : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_EYE_IRIS_NORMAL:    Color = (HitPayload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE ) ? 0.5 + 0.5 * HitPayload.GetEyeIrisNormal()    : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_EYE_CAUSTIC_NORMAL: Color = (HitPayload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_EYE ) ? 0.5 + 0.5 * HitPayload.GetEyeCausticNormal() : 0.0; break;
				case PATH_TRACER_DEBUG_VIZ_HAIR_PRIMITIVE_UV:  Color = (HitPayload.PathTracingShadingModelID == PATH_TRACING_SHADINGMODELID_HAIR) ? float3(HitPayload.GetHairPrimitiveUV(), 0.0) : 0.5; break;
				case PATH_TRACER_DEBUG_VIZ_BACKFACING_NORMAL:
				{
                    Color = 1;
					float K1 = dot(WorldNormal, Ray.Direction);
					float K2 = dot(HitPayload.WorldGeoNormal, reflect(Ray.Direction, WorldNormal));
					if (K1 > 1e-3)
					{
						Color = float3(0, 1, 0);
					}
					else if (K2 < -1e-3)
					{
						Color = float3(1, 0, 0);
					}
					break;
				}
                case PATH_TRACER_DEBUG_VIZ_TRIANGLE_ID:
                case PATH_TRACER_DEBUG_VIZ_INSTANCE_ID:
                case PATH_TRACER_DEBUG_VIZ_CLUSTER_ID:
                    // radiance is populated by the material with the appropriate color
                    Color = HitPayload.Radiance;
                    break;
                case PATH_TRACER_DEBUG_VIZ_DECAL_COVERAGE: Color = lerp(1.0, 0.18, float3(DecalPayload.GetBaseColor().a, DecalPayload.GetWorldNormal().a, DecalPayload.GetMetallicSpecularRoughness().a)); break;
                case PATH_TRACER_DEBUG_VIZ_DECAL_COLOR:    Color = 0.18 * DecalPayload.GetBaseColor().a + DecalPayload.GetBaseColor().xyz; break;
                case PATH_TRACER_DEBUG_VIZ_DECAL_NORMAL:   Color = 0.18 * DecalPayload.GetWorldNormal().a + (DecalPayload.GetWorldNormal().xyz * 0.5 + 0.5 * (1 - DecalPayload.GetWorldNormal().a)); break;
                case PATH_TRACER_DEBUG_VIZ_DECAL_RMS:      Color = 0.18 * DecalPayload.GetMetallicSpecularRoughness().a + DecalPayload.GetMetallicSpecularRoughness().zxy; break; 
                case PATH_TRACER_DEBUG_VIZ_DECAL_RADIANCE: Color = 0.18; ColorAdd = DecalPayload.GetEmissive(); break; // color added after shading
                case PATH_TRACER_DEBUG_VIZ_DECAL_RESPONSE_MASK:
                {
                    Color = lerp(0.18, 0.5, float3(
                        PackedPayload.HasDecalResponseColor(),
                        PackedPayload.HasDecalResponseNormal(),
                        PackedPayload.HasDecalResponseRoughness()));
                    break;
                }
                case PATH_TRACER_DEBUG_VIZ_DECAL_USE_DBUFFER:
                {
                    Color = PackedPayload.UsesDBufferLookup() ? 0.5 : 0.18;
                    break;
                }
				default:
				{
					break;
				}
			}
            float FakeShading = 1;
            if (DebugModeRequiresFakeShading())
            {
                if (AmbientOcclusionMaxDistance > 0)
                {
                    FRayDesc AORay;
                    AORay.Origin = TranslatedWorldPos;
                    AORay.Direction = CosineSampleHemisphereWorld(RandomSequenceCreate(uint3(LaunchIndex, View.StateFrameIndex), 0, 1).Get2D(), WorldNormal).xyz;
                    AORay.TMin = 0;
                    AORay.TMax = AmbientOcclusionMaxDistance;
                    ApplyRayBias(AORay.Origin, PackedPayload.HitT, HitPayload.WorldGeoNormal);
                    FakeShading = Luminance(TraceShadowRay(AORay, 1, 0, 0, true));
                }
                else
                {
					// cheap shading, just to give some definition to otherwise "solid color" modes
                    FakeShading = abs(dot(WorldNormal, Ray.Direction));
                }
            }
            Color *= PathThroughput * HitPayload.BSDFOpacity;
            Result.DiffuseColor += Color;
            Result.WorldNormal += Luminance(PathThroughput) * HitPayload.BSDFOpacity * WorldNormal;
            Result.Radiance.rgb += Color * FakeShading + ColorAdd;
			Result.Radiance.a += Luminance(PathThroughput) * ExtraAlpha;
		}

		if (PackedPayload.ShouldOutputDepth() && Result.Z == 0.0)
		{
			const float3 ViewZ = View.ViewToTranslatedWorld[2].xyz;
			const float3 TranslatedWorldPos = Ray.Origin + PackedPayload.HitT * Ray.Direction;
			Result.LinearDepth = dot(TranslatedWorldPos, ViewZ);
			Result.Z = ConvertToDeviceZ(Result.LinearDepth);
			Result.WorldNormal = PackedPayload.UnpackWorldNormal();
		}

		// account for local transparency change
		PathThroughput *= PackedPayload.UnpackTransparencyColor();

		// prepare next step around the loop (if any throughput is left)
		// retrace the exact same ray with TMin one ulp past the hit we just found
		Ray.TMin = ComputeNewTMin(Ray.Origin, Ray.Direction, PackedPayload.HitT);
		if (!(Ray.TMin < Ray.TMax) || all(PathThroughput == 0))
		{
			break;
		}
	}

	if (DebugMode == PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COUNT)
	{
        Result.Radiance.rgb = 0.5;
        if (TransparentHitCount > 0)
        {
            Result.Radiance.rgb = TransparentHitCount > VisualizeMaxCount ? float3(1, 0.1, 1) : BlueGreenRedColorMap(saturate(float(TransparentHitCount - 1) / float(VisualizeMaxCount)));
        }
		if (OpaquePackedPayload.IsHit())
		{
			// add some fake shading to help with definition
			Result.Radiance.rgb *= abs(dot(OpaquePackedPayload.UnpackWorldNormal(), Ray.Direction));
		}
	}

	return Result;
}

#endif // !PATH_TRACER_PRIMARY_RAYS

RWStructuredBuffer<FPathTracingDebugPackedPathState> PathStateData;
RWBuffer<uint> ActivePaths;
RWBuffer<uint> NumPathStates;

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
RWBuffer<uint> NextActivePaths;
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_CAMERA
FPathState LoadPathStateData(uint Index)
{
	FPathTracingDebugPackedPathState Data = PathStateData[Index];
	FPathState Output = (FPathState)0;
	Output.RandSequence.SampleIndex = Data.RandSeqSampleIndex;
	Output.RandSequence.SampleSeed  = Data.RandSeqSampleSeed;
	Output.Radiance = Data.Radiance;
	Output.Alpha = Data.Alpha;
	Output.PackedDiffSpecColor = Data.PackedDiffSpecColor;
	Output.PackedNormalRoughness = Data.PackedNormalRoughness;
	Output.Ray.Origin = Data.RayOrigin;
	Output.Ray.Direction = Data.RayDirection;
	Output.Ray.TMin = 0;
	Output.Ray.TMax = RAY_DEFAULT_T_MAX;
	Output.PathThroughput = Data.PathThroughput;
	Output.PackedRoughnessSigma = Data.PackedRoughnessSigma;
	Output.PackedScatterPhase = Data.PackedScatterPhase;
	return Output;
}
#endif

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
void StorePathStateData(FPathState PathState, uint Index)
{
	FPathTracingDebugPackedPathState Data;
	Data.RandSeqSampleIndex = PathState.RandSequence.SampleIndex;
	Data.RandSeqSampleSeed  = PathState.RandSequence.SampleSeed;
	Data.Radiance = PathState.Radiance;
	Data.Alpha = PathState.Alpha;
	Data.PackedDiffSpecColor = PathState.PackedDiffSpecColor;
	Data.PackedNormalRoughness = PathState.PackedNormalRoughness;
	Data.RayOrigin = PathState.Ray.Origin;
	Data.RayDirection = PathState.Ray.Direction;
	Data.PathThroughput = PathState.PathThroughput;
	Data.PackedRoughnessSigma = PathState.PackedRoughnessSigma;
	Data.PackedScatterPhase = PathState.PackedScatterPhase;
	PathStateData[Index] = Data;
}
#endif


RAY_TRACING_ENTRY_RAYGEN(PathTracingDebugRG)
{
	uint2 DispatchIdx = DispatchRaysIndex().xy;
	const uint2 DispatchDim = DispatchRaysDimensions().xy;

	ResolvedView = ResolveView();

	uint LinearPixelIndex = DispatchIdx.x + DispatchDim.x * DispatchIdx.y;

#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	uint2 LaunchIndex = DispatchIdx + View.ViewRectMin.xy;
	const float2 AAJitter = 0.5; // trace through pixel center
	const float2 ViewportUV = (LaunchIndex + AAJitter) * View.BufferSizeAndInvSize.zw;
	const FRayDesc CameraRay = CreatePrimaryRay(ViewportUV);
#endif

#if PATH_TRACER_PRIMARY_RAYS

#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	const int FirstBounce = 0;
	FPathState PathState = (FPathState) 0;
	PathState.RandSequence = RandomSequenceCreate(uint3(LaunchIndex, View.StateFrameIndex), SampleIndex, SamplesPerPixel);
	PathState.Ray = CameraRay;
	// path state variables (these cary information between bounces)
	PathState.PathThroughput = 1.0;
	PathState.SetPathRoughness(0.0);
	PathState.SetFirstScatterType(PATHTRACER_SCATTER_CAMERA);
	const uint PathIndex = DispatchIdx.x + 65536 * DispatchIdx.y;
#else

	// compare to number of active paths from the previous bounce
	if (LinearPixelIndex >= NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * FirstBounce + 0])
	{
		return; // nothing left to do on this thread
	}
	const uint PathIndex = ActivePaths[LinearPixelIndex];
	FPathState PathState = LoadPathStateData((PathIndex & 65535) + DispatchDim.x * (PathIndex >> 16));

#endif
	if (LinearPixelIndex == 0)
	{
		// write other dimensions for indirect dispatch of the next bounce
		NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1) + 1] = 1;
		NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1) + 2] = 1;
	}

#if PATH_TRACER_KERNEL_TYPE != PATH_TRACER_KERNEL_TAIL
	const bool KeepGoing = PathTracingDebugCore(PathState, FirstBounce);
	const uint2 TextureIndex = uint2(PathIndex & 65535, PathIndex >> 16);
	#if PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_CAMERA
	{
		// Camera ray must write depth right away because we don't want to have to carry DepthZ in the PackedPathState
		PathState.WriteDepth(TextureIndex);
	}
	#endif
	if (KeepGoing)
	{
	    // Add the path to the set of valid paths for the next bounce
		uint PathStateIndex;
		InterlockedAdd(NumPathStates[DISPATCH_INDIRECT_UINT_COUNT * (FirstBounce + 1)], 1, PathStateIndex);
		NextActivePaths[PathStateIndex] = PathIndex;
		StorePathStateData(PathState, (PathIndex & 65535) + DispatchDim.x * (PathIndex >> 16));
	}
	else
	{
		// nothing left to do
		// Accumulate radiance and update pixel variance
		PathState.WritePixel(TextureIndex);
	}
#else // PATH_TRACER_KERNEL_TYPE == PATH_TRACER_KERNEL_TAIL
	for (int Bounce = FirstBounce; Bounce <= MaxBounces; Bounce++)
	{
		if (!PathTracingDebugCore(PathState, Bounce))
		{
			// kernel had nothing more to do
			break;
		}
	}
	// Accumulate radiance and update pixel variance
	const uint2 TextureIndex = uint2(PathIndex & 65535, PathIndex >> 16);
	PathState.WritePixel(TextureIndex);
#endif

#else
	FPathTracingResult Result = PathTracingDebugCore(CameraRay, LaunchIndex);
	// Write to output textures
	RWSceneColor[DispatchIdx] = Result.Radiance;	
	RWSceneDepth[DispatchIdx] = Result.Z;
	RWSceneDiffuseColor[DispatchIdx] = float4(Result.DiffuseColor.xyz, 1.0f);
	RWSceneSpecularColor[DispatchIdx] = float4(Result.SpecularColor.xyz, 1.0f);
	RWSceneNormal[DispatchIdx] = float4(Result.WorldNormal, Result.Roughness); // TODO: which copy of roughness is actually used?
	RWSceneRoughness[DispatchIdx] = Result.Roughness;
	RWSceneLinearDepth[DispatchIdx] = Result.LinearDepth;
#endif
}
