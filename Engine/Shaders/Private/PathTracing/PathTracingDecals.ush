// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingCommon.ush"
#include "../RayTracing/RayTracingCommon.ush"
#include "../RayTracing/RayTracingDecalGrid.ush"

RaytracingAccelerationStructure DecalTLAS;
float DecalRoughnessCutoff;
float MeshDecalRoughnessCutoff;
float MeshDecalBias;
uint EnableDBuffer;

void EvaluateDecals(float3 RayDirection, uint InstanceInclusionMask, float HitT, float PathRoughness, inout FDecalShaderPayload DecalPayload)
{
    if (PathRoughness < MeshDecalRoughnessCutoff)
    {
        FRayDesc DecalRay = (FRayDesc)0;
        // go a bit further than the current hit, to handle decals partially inside the model
    #if DECAL_USE_REVERSE_CULLING
        DecalRay.Direction = -RayDirection;
    #else
        DecalRay.Direction = RayDirection;
    #endif
        DecalRay.Origin = DecalPayload.TranslatedWorldPos - DecalRay.Direction * MeshDecalBias;
        DecalRay.TMin = 0.0f;
        DecalRay.TMax = HitT + MeshDecalBias;

        const uint MissShaderIndex = 0; // TODO: do we need anything here?
        for (;;)
        {
            DecalPayload.SetMiss();
            TraceRay(
                DecalTLAS,
    #if DECAL_USE_REVERSE_CULLING
                RAY_FLAG_CULL_FRONT_FACING_TRIANGLES,
    #else
                RAY_FLAG_CULL_BACK_FACING_TRIANGLES,
    #endif
                InstanceInclusionMask,
                RAY_TRACING_SHADER_SLOT_MATERIAL,
                RAY_TRACING_NUM_SHADER_SLOTS,
                MissShaderIndex,
                DecalRay.GetNativeDesc(),
                DecalPayload);
    #if NEED_TMIN_WORKAROUND // extra safety - discard hit if not conforming
            if (DecalPayload.HitT <= DecalRay.TMin)
            {
                DecalPayload.HitT = -1.0; 
            }
    #endif
            if (DecalPayload.IsMiss())
            {
                break;
            }

            // prepare next step around the loop
            // retrace the exact same ray with TMin one ulp past the hit we just found
            DecalRay.TMin = ComputeNewTMin(DecalRay.Origin, DecalRay.Direction, DecalPayload.HitT);
        }
    }
#if PLATFORM_SUPPORTS_CALLABLE_SHADERS
    if (PathRoughness < DecalRoughnessCutoff)
    {
        FDecalLoopCount DecalLoopCount = DecalGridLookup(DecalPayload.TranslatedWorldPos);
        for (uint Index = 0, Num = DecalLoopCount.NumDecals; Index < Num; ++Index)
        {
            uint DecalId = GetDecalId(Index, DecalLoopCount);
            CallShader(DecalId, DecalPayload);
        }
    }
#endif
}

bool HasDecalData(FDecalShaderPayload DecalPayload)
{
	return min3(DecalPayload.GetBaseColor().a, DecalPayload.GetWorldNormal().a, DecalPayload.GetMetallicSpecularRoughness().a) < 1.0;
}

void ApplyDecal(FDecalShaderPayload DecalPayload, inout FPackedPathTracingPayload PackedPayload, float PathRoughness)
{
	{
		float3 Radiance = PackedPayload.UnpackRadiance();
		Radiance += DecalPayload.GetEmissive();
		PackedPayload.PackRadiance(Radiance);
	}

	float4 DecalBaseColor = DecalPayload.GetBaseColor();
	float4 DecalWorldNormal = DecalPayload.GetWorldNormal();
	float4 DecalMetallicSpecularRoughness = DecalPayload.GetMetallicSpecularRoughness();

	bool bHasColorResponse = PackedPayload.HasDecalResponseColor() && DecalBaseColor.a < 1.0;
	bool bHasNormalResponse = PackedPayload.HasDecalResponseNormal() && DecalWorldNormal.a < 1.0;
	bool bHasRoughnessResponse = PackedPayload.HasDecalResponseRoughness() && DecalMetallicSpecularRoughness.a < 1.0;

	if (!bHasColorResponse && !bHasNormalResponse && !bHasRoughnessResponse)
	{
		// nothing to do
		return;
	}

	float4 RoughnessData = PackedPayload.UnpackRoughnessAniso();
	float Roughness = RoughnessData.x;
	float3 DiffuseColor = PackedPayload.UnpackDiffuseColor();
	float3 SpecularColor = PackedPayload.UnpackSpecularColor();
	float Metallic = F0RGBToMetallic(SpecularColor);
	float Specular = F0RGBToDielectricSpecular(SpecularColor);
	float3 BaseColor = lerp(DiffuseColor, SpecularColor, Metallic);

	if (bHasColorResponse)
	{
		BaseColor = BaseColor * DecalBaseColor.a + DecalBaseColor.rgb;
	}

	if (bHasNormalResponse)
	{
		// We normalize the normal to get smoother visual result (it helps to avoid having D_GGX explodes toward infinity, and matches ApplyDBufferData(...))
		float3 WorldNormal = PackedPayload.UnpackWorldNormal();
		WorldNormal = normalize(WorldNormal * DecalWorldNormal.a + DecalWorldNormal.rgb);
		PackedPayload.PackWorldNormal(WorldNormal);
	}

	if (bHasRoughnessResponse)
	{
		Metallic  = Metallic  * DecalMetallicSpecularRoughness.a + DecalMetallicSpecularRoughness.x;
		Specular  = Specular  * DecalMetallicSpecularRoughness.a + DecalMetallicSpecularRoughness.y;
		Roughness = Roughness * DecalMetallicSpecularRoughness.a + DecalMetallicSpecularRoughness.z;
		Roughness = max(Roughness, PathRoughness);
	}

	if (bHasColorResponse || bHasRoughnessResponse)
	{
		// the encode-decode is potentially lossy, so only do it if we actually need to
		PackedPayload.PackDiffuseColor(BaseColor - BaseColor * Metallic);
		PackedPayload.PackSpecularColor(lerp(DielectricSpecularToF0(Specular), BaseColor, Metallic));
		PackedPayload.PackRoughnessAniso(float4(Roughness, RoughnessData.yzw));
	}
}



#if USE_DBUFFER
// This bit of code is not easily expressed as a method, so inject it using #define instead. This assumes the surrounding variables will be named a certain way
// TODO: Is there a more elegant way to refactor this?

// SlabRand must be same random value as original opaque trace to get a consistent result to the first call
#define PATH_TRACER_DECAL_PROCESS_DBUFFER(OpaquePayload, ApproximateCaustics, SlabRand)     \
    if (OpaquePayload.UsesDBufferLookup() && HasDecalData(DecalPayload))                    \
    {                                                                                       \
        if (EnableDBuffer)                                                                  \
        {                                                                                   \
            /* Retrace ray with DBuffer data */                                             \
            OpaquePayload = InitPathTracingPayload(                                         \
                PathState.GetFirstScatterType(),                                            \
                ApproximateCaustics * PathState.GetPathRoughness()                          \
            );                                                                              \
            OpaquePayload.SetDBufferA(DecalPayload.GetBaseColor());                         \
            OpaquePayload.SetDBufferB(DecalPayload.GetWorldNormal());                       \
            OpaquePayload.SetDBufferC(DecalPayload.GetMetallicSpecularRoughness());         \
            OpaquePayload.SetStochasticSlabRand(SlabRand);                                  \
            OpaquePayload.SetFlag(PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT);      \
            TraceRay(                                                                       \
                TLAS,                                                                       \
                RayFlags,                                                                   \
                InstanceInclusionMaskOpaque,                                                \
                RAY_TRACING_SHADER_SLOT_MATERIAL,                                           \
                RAY_TRACING_NUM_SHADER_SLOTS,                                               \
                MissShaderIndex,                                                            \
                PathState.Ray.GetNativeDesc(),                                              \
                OpaquePayload);                                                             \
        }                                                                                   \
        else                                                                                \
        {                                                                                   \
            /* The user asked for a dbuffer lookup, but the double-trace is disabled. */    \
            /* Set all the decal response flags so we at least get some kind of response */ \
            /* even if it won't be exactly what the user had in mind. */                    \
            OpaquePayload.SetFlag(                                                          \
                                PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_COLOR |     \
                                PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_NORMAL |    \
                                PATH_TRACING_PAYLOAD_OUTPUT_FLAG_DECAL_RESPONSE_ROUGHNESS); \
        }                                                                                           \
    }

#else

// If DBuffer support is disabled, do nothing.
#define PATH_TRACER_DECAL_PROCESS_DBUFFER(OpaquePayload, ApproximateCaustics, SlabRand) 

#endif
