// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingCommon.ush"


RAY_TRACING_ENTRY_CLOSEST_HIT(PathTracingDefaultOpaqueCHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	// mark hit as solid black
	FPathTracingPayload Payload = (FPathTracingPayload)0;
	Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
	Payload.BSDFOpacity = 1.0;
	PackedPayload = PackPathTracingPayload(Payload);
	PackedPayload.HitT = RayTCurrent();
}

RAY_TRACING_ENTRY_CLOSEST_HIT(PathTracingDefaultHiddenCHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	// mark hit as fully transparent
	FPathTracingPayload Payload = (FPathTracingPayload)0;
	Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
	Payload.TransparencyColor = 1.0;
	PackedPayload = PackPathTracingPayload(Payload);
	PackedPayload.HitT = RayTCurrent();
}

RAY_TRACING_ENTRY_ANY_HIT(PathTracingDefaultHiddenAHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	IgnoreHit();
}
