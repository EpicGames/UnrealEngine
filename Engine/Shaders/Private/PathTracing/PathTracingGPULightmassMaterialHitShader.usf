// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1
#define ENABLE_SKY_LIGHT 1      
#define NEEDS_LIGHTMAP_COORDINATE 0
#ifdef NEEDS_VERTEX_FACTORY_INTERPOLATION
#undef NEEDS_VERTEX_FACTORY_INTERPOLATION
#endif
// Needed for VertexFactoryInterpolate to interpolate attributes from vertices to hit point
#define NEEDS_VERTEX_FACTORY_INTERPOLATION 1

// This should be good enough for path tracing and avoids having to bind an extra buffer
#define EYE_ADAPTATION_PREV_FRAME_EXPOSURE 1

// Ensure that SSS albedo comes through in the material
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0

// Not using precomputed lighting
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING  0

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingShaderUtils.ush"
#include "/Engine/Generated/Material.ush"   
#include "/Engine/Generated/VertexFactory.ush"

#include "/Engine/Private/RayTracing/RayTracingCalcInterpolants.ush"
#include "/Engine/Private/ShadingCommon.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/SubsurfaceProfileCommon.ush"
#include "/Engine/Private/BurleyNormalizedSSSCommon.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingFresnel.ush"

#if SUBSTRATE_ENABLED
#define SUBSTRATE_GPU_LIGHTMASS 1
#include "/Engine/Private/Substrate/Substrate.ush"

#if MATERIAL_IS_SUBSTRATE // SUBSTRATE_TODO: Should this header be guarded from inside instead?
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif

#endif

RAY_TRACING_ENTRY_CLOSEST_HIT(GPULightmassMaterialCHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	PackedPayload.HitT = RayTCurrent();
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

	CurrentPayloadInputFlags = PackedPayload.GetFlags();

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	// this is a newer codepath that is both more flexible and allows for more direct calculation compared to the other codepath
	// TODO: implement such a method for all vertex factories
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FVertexFactoryInterpolantsVSToPS Interpolants;
	float3 GeoNormal = 0;
	CalcInterpolants((FRayCone)0, Attributes, Interpolants, GeoNormal);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif

	FPixelMaterialInputs PixelMaterialInputs;

	const bool bIsFrontFace = HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE;
	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		MaterialParameters.CameraVector = -WorldRayDirection();
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}

#if SUBSTRATE_ENABLED

	float  Coverage = 0;
	float3 TransmittancePreCoverage = 0;
	float3 BaseColorPostCoverage = 0;
	float3 WorldNormal = 0;
	float3 EmissiveLuminance = 0;

#if MATERIAL_IS_SUBSTRATE
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(true /*bForceFullyRough*/, false /*SubstrateStruct.bRoughDiffuse*/, -1/*SubstrateStruct.PeelLayersAboveDepth*/, false/*SubstrateStruct.bRoughnessTracking*/);

	const float3 SurfaceWorldNormal = GeoNormal;
	const float3 V = MaterialParameters.CameraVector;
	FExportResult Export = SubstrateMaterialExportOut(
		Settings,
		SubstratePixelHeader,
		SubstrateData,
		V,
		SurfaceWorldNormal,
		MaterialParameters.WorldPosition_CamRelative,
		0.0f /*MobileShadingPathCurvature*/);

	Coverage = Export.Coverage;
	TransmittancePreCoverage = Export.TransmittancePreCoverage;
	BaseColorPostCoverage = Export.BaseColorPostCoverage;
	WorldNormal = Export.WorldNormal;
	EmissiveLuminance = Export.EmissiveLuminance;
#endif // MATERIAL_IS_SUBSTRATE


#if MATERIALBLENDING_ANY_TRANSLUCENT
	// SUBSTRATE_TODO: Make sure this is correct for all blend modes/glass cases
	float3 TransparencyColor = saturate((1.0f - Coverage) + Coverage * TransmittancePreCoverage);
#else
	float3 TransparencyColor = 0.0;
#endif

	FPathTracingPayload Payload = (FPathTracingPayload)0;

	Payload.BSDFOpacity = Coverage;
	Payload.BaseColor = BaseColorPostCoverage;
	Payload.SubsurfaceColor = 0;							// SUBSTRATE_TODO: Figure out how to support old two-sided foliage approximation
	Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
	Payload.TransparencyColor = TransparencyColor;

	const uint PrimitiveFlags = GetPrimitiveData(MaterialParameters.PrimitiveId).Flags;
	Payload.PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask_FromFlags(PrimitiveFlags);
	Payload.HitT = RayTCurrent();
	if (HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE)
	{
		Payload.SetFrontFace();
	}

#if MATERIAL_IS_SKY
	if (!PackedPayload.IsCameraRay())
	{
		// avoid double counting what was captured by the skylight
		// also avoid noise from hot spots (they can be properly
		// importance sampled if a capturing skylight is added)
		PackedPayload = PackPathTracingPayload(Payload);
		return;
	}
#endif

	Payload.TranslatedWorldPos = DFFastToTranslatedWorld(MaterialParameters.AbsoluteWorldPosition, ResolvedView.PreViewTranslation);
	float GeoNormalSign = MaterialParameters.TwoSidedSign;
	// Because the geometric normal is computed directly in world space
	// it doesn't reflect the sign flip from the object transform, so apply it here
	float3x4 ObjectToWorldXform = ObjectToWorld3x4();
	GeoNormalSign *= sign(determinant(float3x3(ObjectToWorldXform[0].xyz, ObjectToWorldXform[1].xyz, ObjectToWorldXform[2].xyz)));
	Payload.WorldGeoNormal = GeoNormalSign * GeoNormal;
	Payload.WorldNormal = WorldNormal;

	Payload.Radiance = EmissiveLuminance;

#if MATERIAL_TWOSIDED
	Payload.SetMaterialTwoSided();
#else
	if (HitKind() == HIT_KIND_TRIANGLE_BACK_FACE)
	{
		// when viewing the surface from "inside", don't include emission
		Payload.Radiance = 0;
	}
#endif



#else // SUBSTRATE_ENABLED





	FPathTracingPayload Payload = (FPathTracingPayload)0;

	/**
	 * Set common material attributes for both full and simplified materials
	 **/
	uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);

#if MATERIALBLENDING_ALPHACOMPOSITE
	Payload.BSDFOpacity = 1.0;
	Payload.TransparencyColor = 1.0 - GetMaterialOpacity(PixelMaterialInputs);
#elif MATERIALBLENDING_ALPHAHOLDOUT
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0 - GetMaterialOpacity(PixelMaterialInputs);
	Payload.SetHoldout();
	HLSL_STATIC_ASSERT(MATERIAL_SHADINGMODEL_UNLIT == 1, "Alpha holdout blend mode requires unlit shading model");
	ShadingModelID = SHADINGMODELID_UNLIT;
#elif MATERIALBLENDING_TRANSLUCENT
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0 - Payload.BSDFOpacity;
#elif MATERIALBLENDING_ADDITIVE
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0;
#elif MATERIALBLENDING_MODULATE
	Payload.BSDFOpacity = 0.0;
	Payload.TransparencyColor = GetMaterialEmissive(PixelMaterialInputs);
	HLSL_STATIC_ASSERT(MATERIAL_SHADINGMODEL_UNLIT == 1 ||
	                   MATERIAL_SHADINGMODEL_DEFAULT_LIT, "Modulate blend mode requires unlit shading model");
	ShadingModelID = SHADINGMODELID_UNLIT;
#elif MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK
	// dithering emulates real transparency, so switch to translucent
	// NOTE: the raster path technically takes into account the opacity mask clip value, so the effective transparency should be:
	//        saturate(MaskRaw - ClipValue + 0.5)
	// (See derivation in DitheredOpacityMaskToOpacity)
	// However this behavior is surprising to most users and does not exactly match the rasterizer anyway due to how the realtime AA
	// code performs blending.
	// Since the goal of dithered opacity is to emulate ordinary transparency, just use the mask input as opacity directly and
	// ignore the configured clip value.
	Payload.BSDFOpacity = saturate(GetMaterialMaskInputRaw(PixelMaterialInputs));
	Payload.TransparencyColor = 1.0 - Payload.BSDFOpacity;
#elif MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	Payload.BSDFOpacity = 1.0;
	Payload.TransparencyColor = 0.0;
#else
#error Unknown material blending mode
#endif

	// fetch primitive flags only once
	// TODO: would be nice to keep this inside MaterialParameters as it is also needed there as well
	const uint PrimitiveFlags = GetPrimitiveData(MaterialParameters.PrimitiveId).Flags;

	Payload.PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask_FromFlags(PrimitiveFlags);

	Payload.HitT = RayTCurrent();
	if (HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE)
	{
		Payload.SetFrontFace();
	}

#if MATERIAL_IS_SKY
	if (!PackedPayload.IsCameraRay())
	{
		// avoid double counting what was captured by the skylight
		// also avoid noise from hot spots (they can be properly
		// importance sampled if a capturing skylight is added)
		PackedPayload = PackPathTracingPayload(Payload);
		return;
	}
#endif

	// Store the results in local variables and reuse instead of calling the functions multiple times.
	half3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	half  Specular = GetMaterialSpecular(PixelMaterialInputs);
	half Roughness = GetMaterialRoughness(PixelMaterialInputs);
	float Ior = 0.0;

	Payload.TranslatedWorldPos = DFFastToTranslatedWorld(MaterialParameters.AbsoluteWorldPosition, ResolvedView.PreViewTranslation);
	float GeoNormalSign = MaterialParameters.TwoSidedSign;
	// Because the geometric normal is computed directly in world space
	// it doesn't reflect the sign flip from the object transform, so apply it here
	float3x4 ObjectToWorldXform = ObjectToWorld3x4();
	GeoNormalSign *= sign(determinant(float3x3(ObjectToWorldXform[0].xyz, ObjectToWorldXform[1].xyz, ObjectToWorldXform[2].xyz)));
	Payload.WorldGeoNormal = GeoNormalSign * GeoNormal;
	Payload.WorldNormal = MaterialParameters.WorldNormal;
#if !MATERIAL_TANGENTSPACENORMAL
	// already flipped if the material was in tangent space, so add the flip if it wasn't
	Payload.WorldNormal *= MaterialParameters.TwoSidedSign;
#endif

	/**
	 * Set material attributes for simplified materials
	 **/
#if MATERIALBLENDING_TRANSLUCENT && !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	 // Force opacity to 0 so it is ignored for path hits and only evaluated in AHS as a modulation to throughput
	Payload.BSDFOpacity = 0;
	Payload.TransparencyColor = 1;
#else
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1 - Payload.BSDFOpacity;
#endif

	// Anything unsupported will be forced to default lit
#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	if (ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_FOLIAGE;
		Payload.SubsurfaceColor = GetMaterialSubsurfaceData(PixelMaterialInputs).rgb;
	}
	else
#endif
#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
		if (ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
		{
			const float3 Transmission = GetThinTranslucentMaterialOutput0(MaterialParameters);
			const float3 V = WorldRayDirection();
			const float3 N = normalize(MaterialParameters.WorldNormal);
			const float VoN = abs(dot(V, N));
			// simplified logic with no bending at the interface and no fresnel
			Payload.TransparencyColor *= pow(Transmission, 1.0 / VoN);
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
		}
		else
#endif
#if MATERIAL_SHADINGMODEL_UNLIT
			if (ShadingModelID == SHADINGMODELID_UNLIT)
			{
				Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
			}
			else
#endif
			{
				// Redirect translucent blending mode to thin translucent
#if MATERIALBLENDING_TRANSLUCENT
				Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_THIN_GLASS;
#else
				Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
#endif
			}
	Payload.Radiance = GetMaterialEmissive(PixelMaterialInputs);
	Payload.Radiance *= Payload.BSDFOpacity; // premultiply
#if MATERIAL_TWOSIDED
	Payload.SetMaterialTwoSided();
#else
	if (HitKind() == HIT_KIND_TRIANGLE_BACK_FACE)
	{
		// when viewing the surface from "inside", don't include emission
		Payload.Radiance = 0;
	}
#endif
	Payload.BaseColor = BaseColor;



#endif // SUBSTRATE_ENABLED



	PackedPayload = PackPathTracingPayload(Payload);
}

#if USE_MATERIAL_ANY_HIT_SHADER

RAY_TRACING_ENTRY_ANY_HIT(GPULightmassMaterialAHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
#if MATERIALBLENDING_MODULATE || (MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK) || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT
	if (!PackedPayload.IsVisibilityRay())
	{
		// not a shadow ray -- don't need to run the AHS logic
		return;
	}
	if (PackedPayload.CheckDuplicateAHS(RayTCurrent()))
	{
		// We just processed this exact hit, don't double-count it
		IgnoreHit();
		return;
	}
#endif

	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

	CurrentPayloadInputFlags = PackedPayload.GetFlags();

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	// this is a newer codepath that is both more flexible and allows for more direct calculation compared to the other codepath
	// TODO: implement such a method for all vertex factories
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FVertexFactoryInterpolantsVSToPS Interpolants;
	float3 GeoNormal = 0;
	CalcInterpolants((FRayCone)0, Attributes, Interpolants, GeoNormal);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif

	FPixelMaterialInputs PixelMaterialInputs;

	const bool bIsFrontFace = HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE;

	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);

		MaterialParameters.CameraVector = -WorldRayDirection();

		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}

#if MATERIALBLENDING_MASKED && !MATERIAL_DITHER_OPACITY_MASK
	// Regardless of payload flags -- we always apply this 
	if (GetMaterialMask(PixelMaterialInputs) < 0)
	{
		IgnoreHit();
	}
#endif

#if MATERIALBLENDING_MODULATE || (MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK) || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT

#if SUBSTRATE_ENABLED

	float  Coverage = 0;
	float3 TransmittancePreCoverage = 0;
	float3 BaseColorPostCoverage = 0;
	float3 WorldNormal = 0;
	float3 EmissiveLuminance = 0;

#if MATERIAL_IS_SUBSTRATE
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();


	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(true /*bForceFullyRough*/, false /*SubstrateStruct.bRoughDiffuse*/, -1/*SubstrateStruct.PeelLayersAboveDepth*/, false/*SubstrateStruct.bRoughnessTracking*/);

	const float3 SurfaceWorldNormal = GeoNormal;
	const float3 V = MaterialParameters.CameraVector;
	FExportResult Export = SubstrateMaterialExportOut(
		Settings,
		SubstratePixelHeader,
		SubstrateData,
		V,
		SurfaceWorldNormal,
		MaterialParameters.WorldPosition_CamRelative,
		0.0f /*MobileShadingPathCurvature*/);

	Coverage = Export.Coverage;
	TransmittancePreCoverage = Export.TransmittancePreCoverage;
	BaseColorPostCoverage = Export.BaseColorPostCoverage;
	WorldNormal = Export.WorldNormal;
	EmissiveLuminance = Export.EmissiveLuminance;

	float3 Transparency = saturate((1.0f - Coverage) + Coverage * TransmittancePreCoverage);
#else
	// Invalid material
	// SUBSTRATE_TODO: Investigate which materials land in this case, and if there is any need to process them more fully
	float3 Transparency = 0;
#endif

#else // SUBSTRATE_ENABLED

	// the following blend modes need to process shadows after having executed the material
#if MATERIALBLENDING_MODULATE
	const float3 Transparency = GetMaterialEmissive(PixelMaterialInputs);
#elif MATERIALBLENDING_ALPHACOMPOSITE
	const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
	const float Transparency = 1 - Opacity;
#elif MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK
	// See MATERIAL_DITHER_OPACITY_MASK comment below
	const float Opacity = saturate(GetMaterialMaskInputRaw(PixelMaterialInputs));
	const float Transparency = 1 - Opacity;
#elif MATERIALBLENDING_TRANSLUCENT
	const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
	// GPULM has backwards compatibility with some cpu lightmass tricks
	float3 Transparency = 1 - Opacity;
	uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	if (ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
	{
		float3 Transmission = GetThinTranslucentMaterialOutput0(MaterialParameters);
		float Ior = 0.0;
		float3 V = WorldRayDirection();
		float3 N = normalize(MaterialParameters.WorldNormal);
		float VoN = abs(dot(V, N));
		// simplified logic with no bending at the interface and no fresnel
		Transparency *= pow(Transmission, 1.0 / VoN);
	}
	else
#endif // MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
#if MATERIAL_SHADINGMODEL_DEFAULT_LIT
	if (ShadingModelID == SHADINGMODELID_DEFAULT_LIT)
	{
		// CPU Lightmass translucency behavior emulation
		Transparency = lerp(float3(1, 1, 1), GetMaterialBaseColor(PixelMaterialInputs), Opacity);
	}
	else
#endif// MATERIAL_SHADINGMODEL_DEFAULT_LIT
	{
		// base case for shadingmodel if/else
	}
#else // MATERIALBLENDING_*
#error Unhandled blending mode!
#endif

#endif // !SUBSTRATE_ENABLED

	// Update the ray throughput (it is packed simply into the payload since we don't need to carry any other information across hits)
	float3 RayThroughput = PackedPayload.GetRayThroughput();
	RayThroughput *= Transparency;
	PackedPayload.SetRayThroughput(RayThroughput);
	if (any(RayThroughput > 0))
	{
		// if there is energy left in the ray, 
		IgnoreHit();
	}
#endif // MATERIALBLENDING_MODULATE || (MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK) || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT
}

#endif // USE_MATERIAL_ANY_HIT_SHADER
