// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1
#define PATH_TRACER_USE_CLOUD_SHADER 1

#define USE_PRESHADED_UNIFORM_BUFFER 1
#define USE_CALLABLE_SHADERS 1
#define USE_MULTIPLE_VOLUMES 1

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingShaderUtils.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingVolumeSampling.ush"
#include "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesTracingUtils.ush"
#include "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridTypes.ush"
#include "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridUtils.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingVolumeCommon.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingHeterogeneousVolumesForwardDecl.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingHeterogeneousVolumesMaterialCommon.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingHeterogeneousVolumesTrackingCommon.ush"
#include "/Engine/Private/PathTracing/Volume/PathTracingHeterogeneousVolumesPreshadingCommon.ush"

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumeMaterialShader, FPackedPathTracingPayload, PackedPayload)
{
	// Not implemented
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumesIntersectCS, FPackedPathTracingPayload, PackedPayload)
{
	ResolvedView = ResolveView();
	FRayDesc Ray = GetHeterogeneousVolumeCallableShaderRay(PackedPayload);

	// Convert ray to local-space
	float3 WorldOrigin = GetHeterogeneousVolumeCallableShaderWorldPos(Ray.Origin);
	float3 WorldDirection = Ray.Direction;
	float3 LocalOrigin = mul(float4(WorldOrigin, 1), PreshadingUniformBuffer.WorldToLocal).xyz;
	float3 LocalDirection = mul(float4(WorldDirection, 0), PreshadingUniformBuffer.WorldToLocal).xyz;

	float2 LocalRayHitT = IntersectAABB(
		LocalOrigin,
		LocalDirection,
		Ray.TMin,
		Ray.TMax,
		PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent,
		PreshadingUniformBuffer.LocalBoundsOrigin + PreshadingUniformBuffer.LocalBoundsExtent
	);

#if 0
	// Refine intersection interval to include non-zero majorant voxels
	if (LocalRayHitT.y - LocalRayHitT.x > 0)
	{
		// Convert to Majorant voxel-space units
		float3 LocalSpanT0 = LocalOrigin + LocalDirection * LocalRayHitT.x;
		float3 LocalSpanT1 = LocalOrigin + LocalDirection * LocalRayHitT.y;
		float3 LocalBoundsMin = PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent;
		float3 LocalBoundsDiagonal = 2.0 * PreshadingUniformBuffer.LocalBoundsExtent;
		float3 MajorantVoxelSpanT0 = (LocalSpanT0 - LocalBoundsMin) / LocalBoundsDiagonal * PreshadingUniformBuffer.MajorantResolution;
		float3 MajorantVoxelSpanT1 = (LocalSpanT1 - LocalBoundsMin) / LocalBoundsDiagonal * PreshadingUniformBuffer.MajorantResolution;

		float3 MajorantVoxelOrigin = MajorantVoxelSpanT0;
		float3 MajorantVoxelDirection = MajorantVoxelSpanT1 - MajorantVoxelSpanT0;
		float MajorantVoxelTMin = 0.0;
		float MajorantVoxelTMax = length(MajorantVoxelDirection);
		MajorantVoxelDirection /= MajorantVoxelTMax;
		float3 MajorantVoxelDirectionInv = 1.0 / MajorantVoxelDirection;
		float VoxelToLocalScale = (LocalRayHitT.y - LocalRayHitT.x) / MajorantVoxelTMax;

		float3 MajorantVoxelPos = MajorantVoxelOrigin;
		float3 MajorantVoxelBounds;
		MajorantVoxelBounds.x = sign(MajorantVoxelDirection.x) > 0 ? floor(MajorantVoxelPos.x) + 1 : ceil(MajorantVoxelPos.x) - 1;
		MajorantVoxelBounds.y = sign(MajorantVoxelDirection.y) > 0 ? floor(MajorantVoxelPos.y) + 1 : ceil(MajorantVoxelPos.y) - 1;
		MajorantVoxelBounds.z = sign(MajorantVoxelDirection.z) > 0 ? floor(MajorantVoxelPos.z) + 1 : ceil(MajorantVoxelPos.z) - 1;

		float3 MajorantVoxelBoundsHitT = (MajorantVoxelBounds - MajorantVoxelPos) * MajorantVoxelDirectionInv;
		// Remove floating-point rounding error
		float Epsilon = 1.0e-4;
		if (MajorantVoxelBoundsHitT.x <= Epsilon) MajorantVoxelBoundsHitT.x += abs(MajorantVoxelDirectionInv.x);
		if (MajorantVoxelBoundsHitT.y <= Epsilon) MajorantVoxelBoundsHitT.y += abs(MajorantVoxelDirectionInv.y);
		if (MajorantVoxelBoundsHitT.z <= Epsilon) MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);

		// March in voxel space
		float2 MajorantVoxelRayHitT = float2(MajorantVoxelTMax, MajorantVoxelTMin);
		float MajorantVoxelT = 0;
		while (MajorantVoxelT < MajorantVoxelTMax)
		{
			float MajorantVoxelDeltaT = 0.0;
			if (MajorantVoxelBoundsHitT.x < MajorantVoxelBoundsHitT.y)
			{
				if (MajorantVoxelBoundsHitT.x < MajorantVoxelBoundsHitT.z)
				{
					MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.x - MajorantVoxelT;
					MajorantVoxelBoundsHitT.x += abs(MajorantVoxelDirectionInv.x);
				}
				else
				{
					MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.z - MajorantVoxelT;
					MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);
				}
			}
			else
			{
				if (MajorantVoxelBoundsHitT.y < MajorantVoxelBoundsHitT.z)
				{
					MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.y - MajorantVoxelT;
					MajorantVoxelBoundsHitT.y += abs(MajorantVoxelDirectionInv.y);
				}
				else
				{
					MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.z - MajorantVoxelT;
					MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);
				}
			}

			// Clip voxel delta-t by overall voxel ray-length.
			if (MajorantVoxelT + MajorantVoxelDeltaT > MajorantVoxelTMax)
			{
				MajorantVoxelDeltaT = MajorantVoxelTMax - MajorantVoxelT;
			}

			// Convert midpoint to UVW for sampling
			float3 MajorantVoxelEndPos = MajorantVoxelPos + MajorantVoxelDirection * MajorantVoxelDeltaT;
			float3 MajorantVoxelSamplePos = (MajorantVoxelPos + MajorantVoxelEndPos) * 0.5;
			float3 UVW = MajorantVoxelSamplePos / PreshadingUniformBuffer.MajorantResolution;

			int MipLevel = 0;
			float3 Majorant = Texture3DSampleLevel(
				PreshadingUniformBuffer.MajorantTexture,
				PreshadingUniformBuffer.PointTextureSampler,
				UVW,
				MipLevel
			).rgb;

			// Expand hit interval if non-zero majorant voxel
			if (any(Majorant > 0))
			{
				MajorantVoxelRayHitT.x = min(MajorantVoxelRayHitT.x, MajorantVoxelT);
				MajorantVoxelRayHitT.y = max(MajorantVoxelRayHitT.y, MajorantVoxelT + MajorantVoxelDeltaT);
			}

			MajorantVoxelT += MajorantVoxelDeltaT;
			MajorantVoxelPos = MajorantVoxelEndPos;
		}

		float LocalRayHitT0 = LocalRayHitT.x;
		LocalRayHitT.x = LocalRayHitT0 + MajorantVoxelRayHitT.x * VoxelToLocalScale;
		LocalRayHitT.y = LocalRayHitT0 + MajorantVoxelRayHitT.y * VoxelToLocalScale;
	}
#endif

	SetHeterogeneousVolumeCallableShaderHitT(LocalRayHitT, PackedPayload);
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumesGetDensityBoundsCS, FPackedPathTracingPayload, PackedPayload)
{
	// Not implemented
}

// Specialize GetDensity() to locally fetch from the preshaded volume
FVolumeShadedResult HeterogeneousVolumesGetDensity(float3 TranslatedWorldPos, int HeterogeneousVolumesIndex)
{
	float3 SigmaT = 0;
	float3 Albedo = 0;
	float3 Emission = 0;

	SampleVolumeRenderingCoefficients(TranslatedWorldPos, SigmaT, Albedo, Emission);

	// Return struct
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT = SigmaT;
	Result.SigmaSHG = Albedo * SigmaT;
	Result.PhaseG = GetHeterogeneousVolumeCallableShaderPhaseG();
	Result.Emission = Emission;
	return Result;
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumesGetDensityCS, FPackedPathTracingPayload, PackedPayload)
{
	ResolvedView = ResolveView();

	float3 SigmaT = 0;
	float3 Albedo = 0;
	float3 Emission = 0;

	float3 TranslatedWorldPos = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(PackedPayload);
	SampleVolumeRenderingCoefficients(TranslatedWorldPos, SigmaT, Albedo, Emission);

	SetHeterogeneousVolumeCallableShaderSigmaT(SigmaT, PackedPayload);
	SetHeterogeneousVolumeCallableShaderAlbedo(Albedo, PackedPayload);
	SetHeterogeneousVolumeCallableShaderEmission(Emission, PackedPayload);
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumesSampleDistanceCS, FPackedPathTracingPayload, PackedPayload)
{
	ResolvedView = ResolveView();

	FRandomSequence RandSequence = GetHeterogeneousVolumeCallableShaderRandomSequence(PackedPayload);
	FRayDesc Ray = GetHeterogeneousVolumeCallableShaderRay(PackedPayload);
	float3 PathThroughput = GetHeterogeneousVolumeCallableShaderThroughput(PackedPayload);
	float Majorant = GetHeterogeneousVolumeCallableShaderMajorant(PackedPayload);

	// The tracking interval may represent overlapping volumes, so refine it against the active volume bounds
	{
		float3 WorldOrigin = GetHeterogeneousVolumeCallableShaderWorldPos(Ray.Origin);
		float3 WorldDirection = Ray.Direction;
		float3 LocalOrigin = mul(float4(WorldOrigin, 1), PreshadingUniformBuffer.WorldToLocal).xyz;
		float3 LocalDirection = mul(float4(WorldDirection, 0), PreshadingUniformBuffer.WorldToLocal).xyz;

		 float2 LocalRayHitT = IntersectAABB(
			LocalOrigin,
			LocalDirection,
			Ray.TMin,
			Ray.TMax,
			PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent,
			PreshadingUniformBuffer.LocalBoundsOrigin + PreshadingUniformBuffer.LocalBoundsExtent
		);

		 // Invalid intersection
		 if (LocalRayHitT.x >= LocalRayHitT.y)
		 {
			 SetHeterogeneousVolumeCallableShaderThroughput(PathThroughput, PackedPayload);
			 SetHeterogeneousVolumeCallableShaderPdf(1, PackedPayload);
			 SetHeterogeneousVolumeCallableShaderDistance(Ray.TMax, PackedPayload);
			 SetHeterogeneousVolumeCallableShaderMajorant(0, PackedPayload);
			 SetHeterogeneousVolumeCallableShaderIsCollision(false, PackedPayload);
			 return;
		 }
	}

	bool bTransmittanceOnly = false;
	FVolumeTrackingResult Result = HeterogeneousVolumesMajorantBasedTrackingPreshading(
		PathThroughput,
		Ray.Origin,
		Ray.Direction,
		Ray.TMin,
		Ray.TMax,
		bTransmittanceOnly,
		CreateMajorantData(Majorant),
		RandSequence
	);

	SetHeterogeneousVolumeCallableShaderThroughput(Result.Throughput, PackedPayload);
	SetHeterogeneousVolumeCallableShaderPdf(Result.Pdf, PackedPayload);
	SetHeterogeneousVolumeCallableShaderDistance(Result.Distance, PackedPayload);
	SetHeterogeneousVolumeCallableShaderSigmaBar(Result.SigmaBar, PackedPayload);
	SetHeterogeneousVolumeCallableShaderIsCollision(Result.bIsCollision, PackedPayload);

	SetHeterogeneousVolumeCallableShaderRandomSequence(RandSequence, PackedPayload);
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingHeterogeneousVolumesGetTransmittanceCS, FPackedPathTracingPayload, PackedPayload)
{
	ResolvedView = ResolveView();

	FRandomSequence RandSequence = GetHeterogeneousVolumeCallableShaderRandomSequence(PackedPayload);
	FRayDesc Ray = GetHeterogeneousVolumeCallableShaderRay(PackedPayload);
	float3 PathThroughput = GetHeterogeneousVolumeCallableShaderThroughput(PackedPayload);

	bool bTransmittanceOnly = true;
	FMajorantData EmptyOverlappingMajorant = CreateMajorantData(0.0f);
	FVolumeTrackingResult Result = HeterogeneousVolumesMajorantBasedTrackingPreshading(
		PathThroughput,
		Ray.Origin,
		Ray.Direction,
		Ray.TMin,
		Ray.TMax,
		bTransmittanceOnly,
		EmptyOverlappingMajorant,
		RandSequence
	);

	SetHeterogeneousVolumeCallableShaderThroughput(Result.Throughput, PackedPayload);
	SetHeterogeneousVolumeCallableShaderRandomSequence(RandSequence, PackedPayload);
}
