// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "PathTracingCommon.ush"

RaytracingAccelerationStructure TLAS;
RWBuffer<float> RWStartingMediumData;

RAY_TRACING_ENTRY_RAYGEN(PathTracingInitExtinctionCoefficientRG)
{
    const uint RayFlags = RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES;
	const uint MissShaderIndex = 0;
	const uint InstanceInclusionMask = PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_CAMERA_TRANSLUCENT; // Only trace what camera rays can see

	FRayDesc UpRay;
	UpRay.Origin = PrimaryView.TranslatedWorldCameraOrigin;
	UpRay.Direction = float3(0, 0, 1);
	UpRay.TMin = 0.0;
	UpRay.TMax = RAY_DEFAULT_T_MAX;
		
	float3 StartingSigmaT = 0.0;
	float3 StartingSigmaS = 0.0;
	float StartingPhaseG = 0;
	for (;;)
	{
		FPackedPathTracingPayload PackedPayload = InitPathTracingPayload(PATHTRACER_SCATTER_CAMERA, 0.0);
		// TODO: Should we try to average the result of several paths here? If a material is only partially glass, we may not get a consistent answer here
		PackedPayload.SetStochasticSlabRand(0.5);
		TraceRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			RAY_TRACING_SHADER_SLOT_MATERIAL,
			RAY_TRACING_NUM_SHADER_SLOTS,
			MissShaderIndex,
			UpRay.GetNativeDesc(),
			PackedPayload);
#if NEED_TMIN_WORKAROUND // extra safety - discard hit if not conforming
		if (PackedPayload.HitT <= UpRay.TMin)
		{
			PackedPayload.HitT = -1.0;
		}
#endif
		if (PackedPayload.IsMiss())
		{
			// we didn't hit anything
			break;
		}
		FPathTracingPayload HitPayload = UnpackPathTracingPayload(PackedPayload, UpRay);
		if (HitPayload.IsMaterialSolidGlass())
		{
			// Found a solid transmissive medium -- pickup the extinction
			float3 LocalSigmaT = HitPayload.GetExtinction();
			float3 LocalSigmaS = HitPayload.GetGlassAlbedo() * LocalSigmaT;
			float PhaseG = HitPayload.GetGlassPhase();

			float WeightPrev = max(LobeColorToWeight(StartingSigmaT), 0);
			float WeightCurr = max(LobeColorToWeight(LocalSigmaT), 0);

			// increase when we leave a medium (that means we must have been inside before)
			// decrease when we enter a medium (will be cancelled out when we leave)
			StartingSigmaT += HitPayload.IsFrontFace() ? -LocalSigmaT : +LocalSigmaT;
			StartingSigmaS += HitPayload.IsFrontFace() ? -LocalSigmaS : +LocalSigmaS;

			StartingPhaseG = lerp(StartingPhaseG, PhaseG, MISWeightBalanced(WeightCurr, WeightPrev));
		}
		// keep tracing
		UpRay.TMin = ComputeNewTMin(UpRay.Origin, UpRay.Direction, PackedPayload.HitT);
	}
	// clamp against 0 in the unlikely case that we only hit front faces of glass on our way up
	StartingSigmaT = max(StartingSigmaT, 0.0);
	StartingSigmaS = max(StartingSigmaS, 0.0);

	RWStartingMediumData[0] = StartingSigmaT.x;
	RWStartingMediumData[1] = StartingSigmaT.y;
	RWStartingMediumData[2] = StartingSigmaT.z;

	RWStartingMediumData[3] = StartingSigmaS.x;
	RWStartingMediumData[4] = StartingSigmaS.y;
	RWStartingMediumData[5] = StartingSigmaS.z;

	RWStartingMediumData[6] = StartingPhaseG;
}
