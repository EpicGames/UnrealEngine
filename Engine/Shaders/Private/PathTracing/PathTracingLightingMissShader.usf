// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1

#include "PathTracingCommon.ush"
#include "PathTracingShaderUtils.ush"

#define CameraRelativeLightPosition  PathTracingLightFunctionParameters.CameraRelativeLightPosition

#include "/Engine/Generated/Material.ush"

// TODO: if we could just bind loose parameters, we could avoid this workaround for getting the shader parameters needed by the routines in LightFunctionCommon.ush
#define LightFunctionParameters  PathTracingLightFunctionParameters.LightFunctionParameters
#define LightFunctionParameters2 PathTracingLightFunctionParameters.LightFunctionParameters2
#define LightFunctionTranslatedWorldToLight PathTracingLightFunctionParameters.LightFunctionTranslatedWorldToLight
#include "../LightFunctionCommon.ush"
#include "../RayTracing/RayTracingLightFunctionCommon.ush"

RAY_TRACING_ENTRY_MISS(PathTracingLightingMS, FPackedPathTracingPayload, Payload)
{
	Payload.SetMiss();
	ResolvedView = ResolveView();
	CurrentPayloadInputFlags = Payload.GetFlags();
	float3 TranslatedWorldPosition = TranslatedWorldRayOrigin(); // Shadow ray is traced from the shaded point
	float3 Throughput = Payload.GetRayThroughput();
	float3 LightFunctionMult = GetRayTracingLightFunction(TranslatedWorldPosition);

	if (PathTracingLightFunctionParameters.EnableColoredLightFunctions == 0)
	{
		// convert to greyscale for compatibility with the main engine (optionally)
		LightFunctionMult = dot(LightFunctionMult, .3333f);
	}
	Throughput *= LightFunctionMult;
	Payload.SetRayThroughput(Throughput);
}
