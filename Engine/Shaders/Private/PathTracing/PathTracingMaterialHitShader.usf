// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1
#define ENABLE_SKY_LIGHT 1      
#define NEEDS_LIGHTMAP_COORDINATE 0

// This define controls if we use the "g" factor from the SSS profile or not. This is disabled by default because
// we can get a better match to how Metahumans are currently dialed with an isotropic phase function. It may be possible to revisit this
// if we get a better alignment between raster SSS techniques and the PT random walk.
#define USE_SSS_PROFILE_ANISOTROPY 0

#ifdef NEEDS_VERTEX_FACTORY_INTERPOLATION
#undef NEEDS_VERTEX_FACTORY_INTERPOLATION
#endif
// Needed for VertexFactoryInterpolate to interpolate attributes from vertices to hit point
#define NEEDS_VERTEX_FACTORY_INTERPOLATION 1

// This should be good enough for path tracing and avoids having to bind an extra buffer
#define EYE_ADAPTATION_PREV_FRAME_EXPOSURE 1

// Ensure that SSS albedo comes through in the material
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0

// This define happens when we compile material with BlendableGBuffer (legacy) and no substrate nodes are used in a materials.
// As oftoday, we are then going to compile the material with Substrate disabled. But the payload is still a Substrate payload!
// The implementation here will take the legacy code path, but simply encoding the information in the new payload format as needed.
#if (SUBSTRATE_ENABLED && !TEMPLATE_USES_SUBSTRATE) // == TEMPLATE_USES_LEGACYGBUFFERDATA from MaterialTemplate, needs to happen here before other #include in this file.
#define SUBSTRATE_INLINE_SHADING 1
#endif

// The derivatives are used for texture lookup filtering and Substrate's glints
#define USE_ANALYTIC_DERIVATIVES 1

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingShaderUtils.ush"
#include "/Engine/Private/PathTracing/Utilities/PathTracingRIS.ush"
#include "/Engine/Generated/Material.ush"   
#include "/Engine/Generated/VertexFactory.ush"

#include "/Engine/Private/RayTracing/RayTracingCalcInterpolants.ush"
#include "/Engine/Private/ShadingCommon.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/SubsurfaceProfileCommon.ush"
#include "/Engine/Private/BurleyNormalizedSSSCommon.ush"
#include "/Engine/Private/ParticipatingMediaCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingCommon.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingFresnel.ush"

#if SUBSTRATE_ENABLED && (TEMPLATE_USES_LEGACYGBUFFERDATA || MATERIAL_IS_SUBSTRATEHIDDENCONVERSION)
// Either the material was explicitly setup to use legacy data (blendable g-buffer case)
// Or the material is a hidden conversion into a substrate material (adaptive g-buffer case)
// In both cases, for the path tracer we prefer to work with the original data directly as this allows taking the
// well tested legacy path and simply handle the differences in payload encoding locally rather than
// go through the substrate slab structures and reverse engineer the parameters (which is not always possible, particularly for
// cases that require promiting transparency into refraction).
#define USE_LEGACY_CODEPATH_FOR_SUBSTRATE 1
#else
#define USE_LEGACY_CODEPATH_FOR_SUBSTRATE 0
#endif

#if (SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE) || USE_LEGACY_CODEPATH_FOR_SUBSTRATE
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingSubstrateCommon.ush"
#endif

float AdjustMaterialRoughness(float Roughness, float PathRoughness)
{
	// Modify the payload roughness to minimize difficult caustics
	// This is inspired by a trick used in the Arnold renderer:
	//   https://cgg.mff.cuni.cz/~jaroslav/gicourse2010/giai2010-02-marcos_fajardo-slides.pdf (slide 39)
	//   https://www.arnoldrenderer.com/research/Arnold_TOG2018.pdf (section 4.2)
	// NOTE: If approximate caustics are disabled, the path roughness passed in here will be 0.0 which effectively turns off this optimization
	return max(Roughness, PathRoughness);
}

#define PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE    	(MATERIALBLENDING_TRANSLUCENT && (SUBSTRATE_BLENDING_TRANSLUCENT_GREYTRANSMITTANCE || SUBSTRATE_BLENDING_TRANSLUCENT_COLOREDTRANSMITTANCE))

// Translucent shading cases that use a different BSDF response than normal (implemented with SH lighting in the raster pipeline)
#define PATH_TRACER_USE_VOLUMETRIC_SHADING_MODEL 				(TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL)
#define PATH_TRACER_USE_VOLUMETRIC_SHADING_MODEL_DIRECTIONAL 	(TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL)

#if REFRACTION_USE_INDEX_OF_REFRACTION || REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0

#define PATH_TRACER_REFRACTION_ENABLED 1

float GetRefractionIor(FPixelMaterialInputs PixelMaterialInputs, float3 F0)
{
#if REFRACTION_USE_INDEX_OF_REFRACTION
	float Ior = GetMaterialRefractionIOR(GetMaterialRefraction(PixelMaterialInputs));
#else // REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0
	float Ior = DielectricF0RGBToIor(F0);
#endif
	return Ior;
}

float GetRefractionIor(FPixelMaterialInputs PixelMaterialInputs)
{
#if REFRACTION_USE_INDEX_OF_REFRACTION
	float Ior = GetMaterialRefractionIOR(GetMaterialRefraction(PixelMaterialInputs));
#else // REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0
	float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	float  Specular = GetMaterialSpecular(PixelMaterialInputs);
	float3 F0 = ComputeF0(Specular, BaseColor, Metallic);
	float Ior = DielectricF0RGBToIor(F0);
#endif
	return Ior;
}

float FakeCausticsPathFactor(float Roughness, float PathRoughness)
{
	// The heuristic used here is inspired by the following presentations:
	//   - Kulla & Conty: Revisiting Physically Based Shading at Imageworks
	//       https://blog.selfshadow.com/publications/s2017-shading-course/imageworks/s2017_pbs_imageworks_slides_v2.pdf
	//   - Colin Barré-Brisebois - Pica Pica & Nvidia Turing
	//       https://www.ea.com/seed/news/siggraph-2018-picapica-nv-turing

	// NOTE: this returns 0.0 for PathRoughness <= Roughness which accounts for when we are not using ApproximateCaustics
	return (1 - Roughness * Roughness) * saturate(PathRoughness - Roughness);
}

float FakeCaustics(float F0, float Ior, float NoV)
{
	// fake caustics for solid glass, approximates the effect of the boundaries' fresnel
	float Fr = FresnelReflectance(abs(NoV), Ior, F0);
	return Pow2(1 - Fr);
}

float3 ComputeOpticalDepthForHit(float3 LocalSigmaT, bool bIsFrontFace)
{
	// Does the material have any kind of volumetric absorption to apply?
	// Now track the optical thickness so that we can account for Beer's law along the shadow ray
	// TODO: to support lights inside glass we would need to track an extra offset, but this is hopefully an uncommon scenario
	// TODO2: SingleLayerWater needs something similar, but it generally is setup with opaque or masked blend mode and water geometry does not cast shadows ...
	float Distance = RayTCurrent();

	// Front Face: remove contribution from the ray origin to the current hit (assuming the backside will be hit)
	// Back Face:  add contribution from ray origin to current hit (backside) - excess portion not covered by the ray will be removed by the front face hit
	Distance *= bIsFrontFace ? -1.0 : +1.0;

	return LocalSigmaT * Distance;
}

#else

#define PATH_TRACER_REFRACTION_ENABLED 0
#endif // REFRACTION_USE_INDEX_OF_REFRACTION || REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0

float3 DecodeSSSProfileRadius(uint ProfileId, float3 DiffuseColor, float Opacity)
{
	// Burley parameterization
	float3 SurfaceAlbedo = View.SSProfilesTexture.Load(int3(BSSS_SURFACEALBEDO_OFFSET, ProfileId, 0)).xyz;
	float3 DiffuseMeanFreePath = DecodeDiffuseMeanFreePath(View.SSProfilesTexture.Load(int3(BSSS_DMFP_OFFSET, ProfileId, 0))).xyz;
	float WorldUnitScale = DecodeWorldUnitScale(View.SSProfilesTexture.Load(int3(SSSS_TINT_SCALE_OFFSET, ProfileId, 0)).a) * BURLEY_CM_2_MM;

	// Opacity acts as a per-pixel radius multiplier
	// NOTE: this seems backwards? Opacity=0 acts like default-lit while Opacity=1 acts like SSS?
	// NOTE2: Confirm if this interpretation of opacity is correct ...
	float3 SSSRadius = GetMFPFromDMFPApprox(SurfaceAlbedo, DiffuseColor, Opacity * WorldUnitScale * DiffuseMeanFreePath);

	return SSSRadius * BURLEY_MM_2_CM;
}

float DecodeSSSProfileScatteringDistribution(uint ProfileId)
{
	float EncodedScatteringDistribution = View.SSProfilesTexture.Load(int3(SSSS_TRANSMISSION_OFFSET, ProfileId, 0)).z;
	return DecodeScatteringDistribution(EncodedScatteringDistribution);
}

#if SUBSTRATE_ENABLED

float3 GetSimpleVolumeDiffuseColor(float3 DiffuseColor, float3 MeanFreePath)
{
	// See reference in SubstrateEvaluation.ush
	// SUBSTRATE_TODO: In the case of thin materials, we should also include a backscattering diffuse portion
	FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(DiffuseColor, MeanFreePath);
	const float DiffuseToVolumeBlend = SubstrateSlabDiffuseToVolumeBlend(PM);
	const float3 SlabDirectionalAlbedo = IsotropicMediumSlabEnvDirectionalAlbedoALU(PM);
	return lerp(DiffuseColor, SlabDirectionalAlbedo, DiffuseToVolumeBlend);
}

#endif

RAY_TRACING_ENTRY_CLOSEST_HIT(PathTracingMaterialCHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	PackedPayload.HitT = RayTCurrent();
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);


	CurrentPayloadInputFlags = PackedPayload.GetFlags();
	PathTracerSceneDepth = (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH) ? PackedPayload.GetSceneDepth() : SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;

#if USE_DBUFFER && MATERIAL_USES_DECAL_LOOKUP
	if ((CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH) == 0)
	{
		CurrentPayloadDBufferA = PackedPayload.GetDBufferA();
		CurrentPayloadDBufferB = PackedPayload.GetDBufferB();
		CurrentPayloadDBufferC = PackedPayload.GetDBufferC();
	}
#endif

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	// this is a newer codepath that is both more flexible and allows for more direct calculation compared to the other codepath
	// TODO: implement such a method for all vertex factories
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), GetClusterID(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FVertexFactoryInterpolantsVSToPS Interpolants;
	float3 GeoNormal = 0;
	FRaytracingDerivatives RaytracingDerivatives = (FRaytracingDerivatives)0;
	CalcInterpolants(SvPosition, (FRayCone)0, Attributes, Interpolants, RaytracingDerivatives, GeoNormal);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	ApplyRaytracingDerivatives(MaterialParameters, RaytracingDerivatives);
#endif

	FPixelMaterialInputs PixelMaterialInputs;

	const bool bIsFrontFace = IsFrontFaceHit();
	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		MaterialParameters.CameraVector = -WorldRayDirection();
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}

	const float3 V_World = MaterialParameters.CameraVector;
#if SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

#if MATERIAL_IS_SUBSTRATE
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();

	// SUBSTRATE_TODO: These should actually be initialized via the corresponding cvars. To do this properly would require binding FSubstrateSceneData in the path tracing shaders
	// For now, simply match the default substrate settings
	FSubstrateIntegrationSettings IntegrationSettings = InitSubstrateIntegrationSettings(
		false /* bForceFullyRough */,
		true  /* SubstrateStruct.bRoughDiffuse */,
		-1    /* SubstrateStruct.PeelLayersAboveDepth */,
		true  /* SubstrateStruct.bRoughnessTracking */
	);
#if SUBSTRATE_BLENDING_COLOREDTRANSMITTANCEONLY
	const uint BSDFCount = 0;
#else
	const uint BSDFCount = SubstratePixelHeader.SubstrateTree.BSDFCount;
#endif
	float TotalCoverage = 1;
	float3 TotalTransmittancePreCoverage = 0;
	SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V_World, IntegrationSettings, TotalCoverage, TotalTransmittancePreCoverage);
#if PATH_TRACER_REFRACTION_ENABLED
	// transparency is only used for coverage, transmittance happens through refraction (except for shadows potentially, which is handled below)
	float3 Transparency = 1 - TotalCoverage;
#else
	// transparency is used for both coverage and transmittance
	// SUBSTRATE_TODO: Thin case does not seem to be properly handled here, so the TotalTransmittancePreCoverage is recomputed later
	const float3 Transparency = saturate(lerp(1.0, TotalTransmittancePreCoverage, TotalCoverage));
#endif // PATH_TRACER_REFRACTION_ENABLED

#else // MATERIAL_IS_SUBSTRATE
	// This only happens for invalid materials
	const float3 Transparency = 0;
#endif

#endif // SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

	FPathTracingPayload Payload = (FPathTracingPayload)0;

	/**
	 * Set common material attributes for both full and simplified materials
	 **/

#if PATH_TRACER_USE_VOLUMETRIC_SHADING_MODEL
	uint ShadingModelID = SHADINGMODELID_UNLIT;
#else
	uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);
#endif

#if MATERIALBLENDING_ALPHACOMPOSITE
	Payload.BSDFOpacity = 1.0;
	Payload.TransparencyColor = 1.0 - GetMaterialOpacity(PixelMaterialInputs);
#elif MATERIALBLENDING_ALPHAHOLDOUT
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0 - GetMaterialOpacity(PixelMaterialInputs);
	Payload.SetHoldout();
	HLSL_STATIC_ASSERT(MATERIAL_SHADINGMODEL_UNLIT == 1, "Alpha holdout blend mode requires unlit shading model");
	ShadingModelID = SHADINGMODELID_UNLIT;
#elif MATERIALBLENDING_TRANSLUCENT
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0 - Payload.BSDFOpacity;
#elif MATERIALBLENDING_ADDITIVE
	Payload.BSDFOpacity = GetMaterialOpacity(PixelMaterialInputs);
	Payload.TransparencyColor = 1.0;
#elif MATERIALBLENDING_MODULATE
	Payload.BSDFOpacity = 0.0;
	Payload.TransparencyColor = GetMaterialEmissive(PixelMaterialInputs);
	// NOTE: in the case of substrate materials we could have DEFAULT_LIT here
	HLSL_STATIC_ASSERT(MATERIAL_SHADINGMODEL_UNLIT == 1 ||
	                   MATERIAL_SHADINGMODEL_DEFAULT_LIT == 1, "Modulate blend mode requires unlit shading model");
	ShadingModelID = SHADINGMODELID_UNLIT;
#elif MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK
	// dithering emulates real transparency, so switch to translucent
	// NOTE: the raster path technically takes into account the opacity mask clip value, so the effective transparency should be:
	//        saturate(MaskRaw - ClipValue + 0.5)
	// (See derivation in DitheredOpacityMaskToOpacity)
	// However this behavior is surprising to most users and does not exactly match the rasterizer anyway due to how the realtime AA
	// code performs blending.
	// Since the goal of dithered opacity is to emulate ordinary transparency, just use the mask input as opacity directly and
	// ignore the configured clip value.
	Payload.BSDFOpacity = saturate(GetMaterialMaskInputRaw(PixelMaterialInputs));
	Payload.TransparencyColor = 1.0 - Payload.BSDFOpacity;
#elif MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	Payload.BSDFOpacity = 1.0;
	Payload.TransparencyColor = 0.0;
#else
#error Unknown material blending mode
#endif

	// fetch primitive flags only once
	// TODO: would be nice to keep this inside MaterialParameters as it is also needed there as well
	const uint PrimitiveFlags = GetPrimitiveData(MaterialParameters.PrimitiveId).Flags;

	Payload.PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask_FromFlags(PrimitiveFlags);

	Payload.HitT = RayTCurrent();

	if (IsFrontFaceHit())
	{
		Payload.SetFrontFace();
	}

#if MATERIAL_IS_SKY
	if (!PackedPayload.IsCameraRay())
	{
		// avoid double counting what was captured by the skylight
		// also avoid noise from hot spots (they can be properly
		// importance sampled if a capturing skylight is added)
		PackedPayload = PackPathTracingPayload(Payload);
		return;
	}
#endif

	float GeoNormalSign = MaterialParameters.TwoSidedSign;
	// Because the geometric normal is computed directly in world space
	// it doesn't reflect the sign flip from the object transform, so apply it here
	float3x4 ObjectToWorldXform = ObjectToWorld3x4();
	GeoNormalSign *= sign(determinant(float3x3(ObjectToWorldXform[0].xyz, ObjectToWorldXform[1].xyz, ObjectToWorldXform[2].xyz)));
	Payload.WorldGeoNormal = GeoNormalSign * GeoNormal;
	Payload.WorldSmoothNormal = MaterialParameters.TwoSidedSign * TransformTangentNormalToWorld(MaterialParameters.TangentToWorld, float3(0, 0, 1));

#if SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE
	// unless proven otherwise, we are a basic unlit material
	Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;

	float BaseBSDFOpacity = 1; //Payload.BSDFOpacity;

#if !SUBSTRATE_OPAQUE_MATERIAL
	Payload.TransparencyColor = Transparency;
#endif

	if (BSDFCount > 0)
	{

		for (int Index = 0, Num = SubstratePixelHeader.SharedLocalBases.Count; Index < Num; Index++)
		{
			float3 N = SubstratePixelHeader.SharedLocalBases.Normals[Index];
#if !MATERIAL_TANGENTSPACENORMAL
			N *= MaterialParameters.TwoSidedSign;
#endif
			SubstratePixelHeader.SharedLocalBases.Normals[Index] = N;
		}


		int BSDFIdx = 0;
#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

		// Accumulate radiance across all BSDFs
		Payload.Radiance = 0.0;
		Substrate_for_unroll(BSDFIdx = 0, BSDFIdx < BSDFCount, ++BSDFIdx)
		{
			Payload.Radiance += BSDF.LuminanceWeightV * BSDF_GETEMISSIVE(BSDF);
		}

#if !SUBSTRATE_OPTIMIZED_UNLIT 
		// The slab sampler lets us stochastically choose between the different slab models the path tracer supports
		// NOTE: Even if the material has a single substrate slab, we might still need this because refraction is handled separately from SSS
		FRISContext SlabSampler = InitRISContext(PackedPayload.GetStochasticSlabRand());

		Substrate_for_unroll(BSDFIdx = 0, BSDFIdx < BSDFCount, ++BSDFIdx)
		{
			switch (BSDF_GETTYPE(BSDF))
			{
				case SUBSTRATE_BSDF_TYPE_SLAB:
				{
					const float3 WeightV = BSDF.LuminanceWeightV;
					const float3 TransmittanceN = BSDF.TransmittanceAboveAlongN;
					const float CoverageAboveAlongN = BSDF.CoverageAboveAlongN;
					const float3 MaxLobeWeight = BaseBSDFOpacity * WeightV * lerp(1.0f, TransmittanceN, CoverageAboveAlongN); // largest value LobeWeight() could return

					// Is the slab even visible at all?
					if (any(MaxLobeWeight > 0.0))
					{
						const float3 N = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
						const float NoV = saturate(dot(N, V_World));

						float3 DiffuseColor = SLAB_DIFFUSEALBEDO(BSDF);
						float3 F0 = SLAB_F0(BSDF);
						float3 F90 = SLAB_F90(BSDF) * F0RGBToMicroOcclusion(F0);

						float3 RoughnessData = 0;
						RoughnessData.xy = AdjustMaterialRoughness(SLAB_ROUGHNESS(BSDF), PackedPayload.GetPathRoughness());
						// SUBSTRATE_TODO: Support Fresnel82?
						float3 Spec0E = ComputeGGXSpecEnergyTermsRGB(RoughnessData.x, NoV, F0, F90).E;
						float3 Spec1E = Spec0E;
						bool bUseClearCoat = false;
						if (BSDF_GETHASHAZINESS(BSDF))
						{
							const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
							RoughnessData.z = Haziness.Weight;
							RoughnessData.y = AdjustMaterialRoughness(Haziness.Roughness, PackedPayload.GetPathRoughness());
							if (Haziness.bSimpleClearCoat)
							{
								Spec1E = ComputeGGXSpecEnergyTermsRGB(RoughnessData.y, NoV, CLEAR_COAT_F0).E;
								bUseClearCoat = true;
							}
							else
							{
								Spec1E = ComputeGGXSpecEnergyTermsRGB(RoughnessData.y, NoV, F0, F90).E;
							}
						}
						float3 MeanFreePath = 0;
						float PhaseG = 0;
						float3 LegacySubsurfaceColor = 0;
						float LegacyOpacity = 1;
						if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_SIMPLEVOLUME && !BSDF.bIsBottom)
						{
							// SUBSTRATE_TODO: This should really be represented by a different BSDF, but for now we are approximating it with a diffuse lobe
							DiffuseColor = GetSimpleVolumeDiffuseColor(DiffuseColor, SLAB_SSSMFP(BSDF));
						}
						else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
						{
							const uint ProfileId = SLAB_SSSPROFILEID(BSDF);
							const float Scale = SLAB_SSSPROFILERADIUSSCALE(BSDF);
							MeanFreePath = DecodeSSSProfileRadius(ProfileId, Payload.DiffuseColor, Scale);
							#if USE_SSS_PROFILE_ANISOTROPY
							PhaseG = DecodeSSSProfileScatteringDistribution(ProfileId);
							MeanFreePath *= 1 - PhaseG; // Adjust MFP to preserve appearance
							#endif
						}
						else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_WRAP || BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
						{
							// For the old MATERIAL_SHADINGMODEL_SUBSURFACE and SHADINGMODELID_TWOSIDED_FOLIAGE models, we previously had logic to normalize the DiffuseColor and SubsurfaceColor to ensure the sum is < 1
							const float TransmittanceNoL = 1.0f;
							// Reverse the MFP mapping to get back to the original SubsurfaceColor that was provided to SubstrateConvertLegacyMaterialDynamic
							const float3 MFPInCm = SLAB_SSSMFP(BSDF);
							LegacySubsurfaceColor = MeanFreePathToTransmittance(MFPInCm * CENTIMETER_TO_METER, SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
							LegacyOpacity = 1.f - abs(SLAB_SSSPHASEANISOTROPY(BSDF));

							AdjustPathTracingTwoSidedColorBalance(DiffuseColor, LegacySubsurfaceColor);
						}
						else
						{
							// Basic substrate slab, legacy preintegrated skin or SIMPLEVOLUME on the bottom slab, just use MFP directly for the random walk
							MeanFreePath = SLAB_SSSMFP(BSDF);
							PhaseG       = SLAB_SSSPHASEANISOTROPY(BSDF);
						}
						// SUBSTRATE_TODO: Other SSSTYPE cases may result in some portion of a "glass" BSDF
						float FuzzAmount = 0.0;
						float FuzzRoughness = 1.0;
						float3 FuzzColor = 0.0;
						if (BSDF_GETHASFUZZ(BSDF))
						{
							FuzzAmount = SLAB_FUZZ_AMOUNT(BSDF);
							FuzzColor = SLAB_FUZZ_COLOR(BSDF);
							FuzzRoughness = SLAB_FUZZ_ROUGHNESS(BSDF);
							FuzzRoughness = MakeRoughnessSafe(FuzzRoughness, SUBSTRATE_MIN_FUZZ_ROUGHNESS);
						}
						const float FuzzE = ComputeSheenEnergyTerms(FuzzRoughness, NoV).E;
						const float FuzzAttenuation = 1.0 - FuzzAmount * FuzzE;
						float DiffuseWeight = FuzzAttenuation;
						float3 Spec0Albedo = FuzzAttenuation * Spec0E;
						float3 Spec1Albedo = FuzzAttenuation * Spec1E;
						if (bUseClearCoat)
						{
							// clearcoat slab
							float CoatAttenuation = 1.0 - RoughnessData.z * Spec1E.x;
							DiffuseWeight *= CoatAttenuation * (1.0 - Luminance(Spec0E));
							Spec0Albedo *= CoatAttenuation;
							Spec1Albedo *= RoughnessData.z;
						}
						else
						{
							// dual-specular slab
							DiffuseWeight *= 1.0 - Luminance(lerp(Spec0E, Spec1E, RoughnessData.z));
							Spec0Albedo *= 1.0 - RoughnessData.z;
							Spec1Albedo *= RoughnessData.z;
						}

						float Ior = 1; // for refracted lobe only
						float3 Transmittance = 0;
						float RefractMix = 0; // percentage mix of the glass lobe
					#if PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE
						if (BSDF.bIsBottom)
						{
						#if PATH_TRACER_REFRACTION_ENABLED
							Ior = GetRefractionIor(PixelMaterialInputs, F0);
						#endif
							Transmittance = MeanFreePathToTransmittance(MeanFreePath * CENTIMETER_TO_METER, SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
						#if MATERIAL_ISTHINSURFACE
							RefractMix = saturate(Luminance(Transmittance));
						#else
							// blend towards Opaque SSS in the range of [0cm, 8cm]
							RefractMix = saturate(max3(MeanFreePath.x, MeanFreePath.y, MeanFreePath.z));
						#endif
						}
					#endif // PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE

						// SUBSTRATE_TODO: Need to account for glass case here for lobe selection ....
						const float3 FuzzAlbedo = FuzzAmount * FuzzE * FuzzColor;
						const float3 SlabAlbedo = MaxLobeWeight * (DiffuseWeight * (DiffuseColor + LegacySubsurfaceColor) + Spec0Albedo + Spec1Albedo + FuzzAlbedo);
						const float3 RefrAlbedo = MaxLobeWeight * (FuzzAttenuation * 1.0 + FuzzAlbedo); // glass has unit albedo
						// If we are dealing witha multi-BSDF material, stochastically decide if we need to write to the payload
						// This means estimating the overal albedo of the slab and selectively writing to the payload
						// NOTE: Technically, some of the albedo calculation are not needed if we aren't picking between slabs
						// We rely on the compiler to get rid of dead code here to avoid diverging the logic too far between single and multi-lobe cases
						float ContribSlab = LobeColorToWeight(SlabAlbedo);
						if (SlabSampler.Accept((1 - RefractMix) * ContribSlab))
						{
							// We've selected this particular slab - now write it into the payload
							// NOTE: This could happen more than once, so be careful to write to all fields, even the unused ones!
							Payload.BSDFOpacity = BaseBSDFOpacity / ContribSlab;
							Payload.WorldNormal = N;
							if (SubstrateGetSharedLocalBasisType(SubstratePixelHeader.SharedLocalBases.Types, BSDFIdx) == SUBSTRATE_BASIS_TYPE_TANGENT)
							{
								Payload.WorldTangent = SubstratePixelHeader.SharedLocalBases.Tangents[BSDF_GETSHAREDLOCALBASISID(BSDF)];
							}
							Payload.WeightV               = BSDF.LuminanceWeightV;
							Payload.TransmittanceN        = BSDF.TransmittanceAboveAlongN;
							Payload.CoverageAboveAlongN   = BSDF.CoverageAboveAlongN;
							Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
							Payload.Ior                   = 0.0;
							Payload.DiffuseColor          = DiffuseColor;
							Payload.SpecularColor         = F0;
							Payload.SpecularEdgeColor     = F90;
							Payload.RoughnessData         = RoughnessData;
							if (bUseClearCoat)
							{
								Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_CLEARCOAT;
							#if CLEAR_COAT_BOTTOM_NORMAL
								Payload.SetBottomCoatNormal(SubstrateGetSimpleCoatBottomNormal(UnpackHaziness(SLAB_HAZINESS(BSDF)).BottomNormalOct, N));
							#endif
							}
							Payload.Anisotropy = BSDF_GETHASANISOTROPY(BSDF) ? SLAB_ANISOTROPY(BSDF) : 0.0;
							Payload.FuzzAmount    = FuzzAmount;
							Payload.FuzzColor     = FuzzColor;
							Payload.FuzzRoughness = FuzzRoughness;
							if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_WRAP)
							{
								// Legacy support for SHADINGMODELID_SUBSURFACE

								// The legacy behavior has both diffuse+sss responses separately, but we can combine them to avoid needing a custom lobe
								// In practice the diffuse and SSS responses are similar, so the difference should be minimal (and is an approximation of the raster behavior anyway)
								// SUBSTRATE_TODO: If we implement stochastic lobe picking _within_ slabs, we could fine tune this to get back to the original behavior exactly
								Payload.DiffuseColor += LegacySubsurfaceColor;

							#if HAVE_GetSubsurfaceMediumMaterialOutput0
								Payload.MeanFreePath = GetSubsurfaceMediumMaterialOutput0(MaterialParameters);
							#else
								const float SSSRadius = (1 - LegacyOpacity) * 10.0; // simple formula, up to 10cm radius
								Payload.MeanFreePath = SSSRadius;
							#endif

							#if HAVE_GetSubsurfaceMediumMaterialOutput1
								float ScatteringDistribution = clamp(GetSubsurfaceMediumMaterialOutput1(MaterialParameters), -0.99f, 0.99f);
								Payload.PhaseG = ScatteringDistribution;
							#else
								Payload.PhaseG = 0;
							#endif
							}
							else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
							{
								// Legacy support for SHADINGMODELID_TWOSIDED_FOLIAGE
								Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_FOLIAGE;
								Payload.SetFoliageTransmissionColor(LegacySubsurfaceColor);
							}
							else
							{
							#if MATERIAL_ISTHINSURFACE
								// material is thin, make sure we don't use random walk
								// SUBSTRATE_TODO: derive the proper BSDF for the internal scattering layer inside the slab
							#else
								// Setup material for random walk
								Payload.MeanFreePath = MeanFreePath;
								Payload.PhaseG       = PhaseG;
							#endif
							}

							// Handle special-case specular features
							#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
								Payload.SpecularProfileId = 0;
								#if SUBSTRATE_COMPLEXSPECIALPATH
								if (BSDF_GETHASSPECPROFILE(BSDF))
								{
									Payload.SpecularProfileId = SLAB_SPECPROFILEID(BSDF);
								}
								#endif
							#endif

							#if SUBSTRATE_PATH_TRACING_GLINTS
								Payload.GlintValue = 1.f;
								#if SUBSTRATE_COMPLEXSPECIALPATH
								if (BSDF_GETHASGLINT(BSDF))
								{
									Payload.GlintValue = SLAB_GLINT_VALUE(BSDF);
									Payload.GlintUV    = SLAB_GLINT_UV(BSDF);
									Payload.GlintUVdx  = SLAB_GLINT_UVDDX(BSDF);
									Payload.GlintUVdy  = SLAB_GLINT_UVDDY(BSDF);
								}
								#endif
							#endif
						}
					#if PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE
						// now test for the presence of a refracted lobe
						float ContribRefr = LobeColorToWeight(RefrAlbedo);
						if (SlabSampler.Accept(RefractMix * ContribRefr))
						{
							// We've selected this particular slab - now write it into the payload
							// NOTE: This could happen more than once, so be careful to write to all fields, even the unused ones!
							Payload.BSDFOpacity = BaseBSDFOpacity / ContribRefr;
							Payload.WorldNormal = N;
							if (SubstrateGetSharedLocalBasisType(SubstratePixelHeader.SharedLocalBases.Types, BSDFIdx) == SUBSTRATE_BASIS_TYPE_TANGENT)
							{
								Payload.WorldTangent = SubstratePixelHeader.SharedLocalBases.Tangents[BSDF_GETSHAREDLOCALBASISID(BSDF)];
							}
							Payload.WeightV               = BSDF.LuminanceWeightV;
							Payload.TransmittanceN        = BSDF.TransmittanceAboveAlongN;
							Payload.CoverageAboveAlongN   = BSDF.CoverageAboveAlongN;
							#if MATERIAL_ISTHINSURFACE
							Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_THIN_GLASS;
							#else
							Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_SOLID_GLASS;
							// Albedo inversion for ambient medium
							// NOTE: we could technically also include IOR and Phase function - but this basic inversion does most of the work
							float3 Color = DiffuseColor;
							// Van de hulst mapping (with improved constants by d'Eon, see formula 48.20)
							const float b1 = 1.0 / 0.100392;
							const float b2 = 1.21573;
							const float3 t1 = (0.5 * b1 + 0.5) + Color * (0.5 * b2 * b1);
							const float3 t2 = b1 - Color * b1;
							const float3 s = t1 - sqrt(t1 * t1 - t2);
							DiffuseColor = 1.0 - s * s;
							#endif
							Payload.DiffuseColor          = DiffuseColor; // SUBSTRATE_TODO: Add albedo inversion for solid case?
							Payload.SpecularColor         = F0;
							Payload.SpecularEdgeColor     = F90;
							Payload.RoughnessData         = RoughnessData;
							Payload.Anisotropy = BSDF_GETHASANISOTROPY(BSDF) ? SLAB_ANISOTROPY(BSDF) : 0.0;
							Payload.FuzzAmount    = FuzzAmount;
							Payload.FuzzColor     = FuzzColor;
							Payload.FuzzRoughness = FuzzRoughness;
							#if MATERIAL_ISTHINSURFACE
							// Convert to transmittance color now since we won't be using the raw extinction coeffient
							Payload.SetTransmittanceColor(exp(-MeanFreePath * SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM));
							#else
							Payload.MeanFreePath  = MeanFreePath;
							#endif
							Payload.PhaseG        = PhaseG;
							Payload.Ior           = Ior;
							// SUBSTRATE_TODO: should these be supported in the Glass case?
						#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
							Payload.SpecularProfileId = 0;
						#endif
						#if SUBSTRATE_PATH_TRACING_GLINTS
							Payload.GlintValue = 1.f;
						#endif

							// Material is configured as a thin surface, and refraction is disabled
							#if MATERIAL_ISTHINSURFACE && PATH_TRACER_REFRACTION_ENABLED == 0
							{
								// SUBSTRATE_TODO: Do we really want to keep this? Should maybe always use refraction path?
								float3 Transmission = Payload.GetTransmittanceColor();
								const float NoV = saturate(dot(Payload.WorldNormal, V_World));
								float3 Tr = ComputeThinSlabWeights(Transmission, NoV, 0.0, F0RGBToF0(Payload.SpecularColor)).Transmitted;
								Payload.TransparencyColor += TotalCoverage * Tr;
							}
							#endif
						}
					#endif // PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE
					}
					break;
				}
				// NOTE: The cases below are for back-compatibility and cannot be layered
				case SUBSTRATE_BSDF_TYPE_EYE:
				{
					SlabSampler.Accept(1); // dummy, avoids special case below
					Payload.WorldNormal = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
					Payload.DiffuseColor = EYE_DIFFUSEALBEDO(BSDF);
					Payload.SpecularColor = EYE_F0(BSDF);
					// NOTE: EYE_F90 is always 1.0 (does not exist in legacy model), so don't bother with it
					Payload.SetEyeRoughness(AdjustMaterialRoughness(EYE_ROUGHNESS(BSDF), PackedPayload.GetPathRoughness()));

					Payload.MeanFreePath = 0;
					Payload.PhaseG = 0;
					if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
					{
						Payload.MeanFreePath = DecodeSSSProfileRadius(EYE_SSSPROFILEID(BSDF), Payload.DiffuseColor, 1.0);
					}

					float IrisMask = EYE_IRISMASK(BSDF);
					float IrisDistance = EYE_IRISDISTANCE(BSDF);

					// see logic in non-substrate code below
					const float3 PlaneNormal = normalize(EYE_IRISPLANENORMAL(BSDF));
					const float3 CausticNormal = normalize(lerp(PlaneNormal, -Payload.WorldNormal, IrisMask * IrisDistance));

					Payload.SetEyeCausticNormal(CausticNormal);
					Payload.SetEyeIrisMask(IrisMask);
					Payload.SetEyeIrisNormal(normalize(EYE_IRISNORMAL(BSDF)));
					Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_EYE;
				#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
					Payload.SpecularProfileId = 0;
				#endif
				#if SUBSTRATE_PATH_TRACING_GLINTS
					Payload.GlintValue = 1.f;
				#endif
					break;
				}
				case SUBSTRATE_BSDF_TYPE_HAIR:
				{
					SlabSampler.Accept(1); // dummy, avoids special case below
					// clamp the roughness to whatever came along the path
					Payload.SetBaseColor(HAIR_BASECOLOR(BSDF));
					Payload.SetHairLongitudinalRoughness(AdjustMaterialRoughness(HAIR_ROUGHNESS(BSDF), PackedPayload.GetPathRoughness()));
					Payload.SetHairAzimuthalRoughness(HAIR_SCATTER(BSDF)); // SUBSTRATE_TODO: Map from scatter param to azimuthal roughness here instead of in the raygen
					Payload.SetHairSpecular(HAIR_SPECULAR(BSDF));

					#if HAIR_STRAND_MESH_FACTORY
					Payload.WorldSmoothNormal = Payload.WorldNormal = Payload.WorldGeoNormal;
					Payload.WorldTangent      = MaterialParameters.TangentToWorld[2];
					Payload.SetHairPrimitiveUV(MaterialParameters.HairPrimitiveUV);
					#else
					Payload.SetHairPrimitiveUV(0.5);
					#endif
					Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_HAIR;
					#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
					Payload.SpecularProfileId = 0;
					#endif
					#if SUBSTRATE_PATH_TRACING_GLINTS
					Payload.GlintValue = 1.f;
					#endif
					break;
				}
				case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
				{
					SlabSampler.Accept(1); // dummy, avoids special case below
					Payload.WorldNormal = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
					Payload.BSDFOpacity = 1;
					Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_SOLID_GLASS;
					float3 BaseColor = SLW_BASECOLOR(BSDF);
					float Metallic = SLW_METALLIC(BSDF);
					float Specular = SLW_SPECULAR(BSDF);
					float Roughness = SLW_ROUGHNESS(BSDF);
					float3 Extinction = 0.0;
					#if SUBSTRATE_INLINE_SINGLELAYERWATER

					//float Opacity = SLW_TOPMATERIALOPACITY(BSDF); // TODO: Account for this with an extra random decision?
					//float3 WaterAlbedo = SLW_WATERALBEDO(BSDF); // TODO: Account for scattering in refractive mediums
					Extinction = SLW_WATEREXTINCTION(BSDF);
					//float PhaseG = SLW_WATERPHASEG(BSDF);
					//float3 ColorBehindWater = SLW_COLORSCALEBEHINDWATER(BSDF);
					#endif
					Payload.WeightV             = BSDF.LuminanceWeightV;
					Payload.TransmittanceN      = BSDF.TransmittanceAboveAlongN;
					Payload.CoverageAboveAlongN = BSDF.CoverageAboveAlongN;

					Payload.DiffuseColor = ComputeDiffuseAlbedo(BaseColor, Metallic);
					Payload.SpecularColor = ComputeF0(Specular, BaseColor, Metallic);
					Payload.RoughnessData.x = AdjustMaterialRoughness(Roughness, PackedPayload.GetPathRoughness());
					Payload.RoughnessData.yz = 0;
					Payload.Ior = DielectricF0ToIor(DielectricSpecularToF0(Specular));
					Payload.MeanFreePath = rcp(max(Extinction, 1e-5));
					Payload.FuzzAmount = 0;
					Payload.FuzzRoughness = 0;
					Payload.FuzzColor = 0;
					#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
					Payload.SpecularProfileId = 0;
					#endif
					#if SUBSTRATE_PATH_TRACING_GLINTS
					Payload.GlintValue = 1.f;
					#endif
					break;
				}
				case SUBSTRATE_BSDF_TYPE_TOON:
				{
					Payload.BSDFOpacity = BaseBSDFOpacity;
					Payload.WorldNormal = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
					if (SubstrateGetSharedLocalBasisType(SubstratePixelHeader.SharedLocalBases.Types, BSDFIdx) == SUBSTRATE_BASIS_TYPE_TANGENT)
					{
						Payload.WorldTangent = SubstratePixelHeader.SharedLocalBases.Tangents[BSDF_GETSHAREDLOCALBASISID(BSDF)];
					}
					Payload.WeightV               = BSDF.LuminanceWeightV;
					Payload.TransmittanceN        = BSDF.TransmittanceAboveAlongN;
					Payload.CoverageAboveAlongN   = BSDF.CoverageAboveAlongN;
					Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
					Payload.Ior                   = 0.0f;
					Payload.DiffuseColor          = float3(1,0,1); // TOON_TODO
					Payload.SpecularColor         = 0.0f;
					Payload.SpecularEdgeColor     = 0.0f;
					Payload.RoughnessData         = 0.0f;

					// Handle special-case specular features
					Payload.Anisotropy			= 0.0f;
					Payload.FuzzAmount			= 0.0f;
					Payload.FuzzColor			= 0.0f;
					Payload.FuzzRoughness		= 0.0f;
					#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
					Payload.SpecularProfileId	= 0.0f;
					#endif
					#if SUBSTRATE_PATH_TRACING_GLINTS
					Payload.GlintValue			= 1.0f;
					#endif
					break;
				}
			}  // switch on slab tyype
		} // for loop over BSDFs

		// normalize the weight for picking a given slab
		Payload.BSDFOpacity *= SlabSampler.GetNormalization();
#undef  BSDF
#endif // !SUBSTRATE_OPTIMIZED_UNLIT
	} // if (BSDFCount > 0)


#else // SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

	Payload.Radiance = GetMaterialEmissive(PixelMaterialInputs);
	Payload.Radiance *= Payload.BSDFOpacity; // pre-multiply
	// NOTE: Emissive contribution is turned off below for non-two-sided materials

	Payload.WorldNormal = MaterialParameters.WorldNormal;
#if !MATERIAL_TANGENTSPACENORMAL
	// already flipped if the material was in tangent space, so add the flip if it wasn't
	Payload.WorldNormal *= MaterialParameters.TwoSidedSign;
#endif

	// Store the results in local variables and reuse instead of calling the functions multiple times.
	half3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	half  Specular = GetMaterialSpecular(PixelMaterialInputs);
	half Roughness = GetMaterialRoughness(PixelMaterialInputs);
	float Ior = 0.0;
#if MATERIALBLENDING_TRANSLUCENT && PATH_TRACER_REFRACTION_ENABLED && (MATERIAL_SHADINGMODEL_DEFAULT_LIT || MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT)
	// NOTE: only default-lit and thin-translucent use ior, the other material models only support plain transparency
	// This is an attempt to limit the complexity of the material eval/sample API which must take into account an extra lobe if supporting refraction
	Ior = GetRefractionIor(PixelMaterialInputs);
#endif

	Payload.DiffuseColor = ComputeDiffuseAlbedo(BaseColor, Metallic);
	Payload.SpecularColor = ComputeF0(Specular, BaseColor, Metallic);
	Payload.RoughnessData = float3(Roughness, Roughness, 0);
	// reset unused features
	Payload.SpecularEdgeColor = 1;
	Payload.WeightV = 1;
	Payload.CoverageAboveAlongN = 0;
	Payload.TransmittanceN = 1;
	#if SUBSTRATE_PATH_TRACING_GLINTS
	Payload.GlintValue = 1;
	#endif
	#if SUBSTRATE_PATH_TRACING_SPECULAR_PROFILE
	Payload.SpecularProfileId = 0;
	#endif

#if MATERIAL_USES_ANISOTROPY
	Payload.WorldTangent = CalculateAnisotropyTangent(MaterialParameters, PixelMaterialInputs);
	Payload.Anisotropy = GetMaterialAnisotropy(PixelMaterialInputs);
#endif	

#if HAIR_STRAND_MESH_FACTORY
	Payload.WorldNormal = Payload.WorldGeoNormal;
	Payload.WorldTangent = MaterialParameters.TangentToWorld[2];

	// Repurpose WorldSmoothNormal to encode hair strand direction
	Payload.WorldSmoothNormal = normalize(MaterialParameters.WorldNormal);
#endif

#if PATH_TRACER_USE_VOLUMETRIC_SHADING_MODEL
	if (ShadingModelID == SHADINGMODELID_UNLIT)
	{
		// encode the directionality using Anisotropy: remap [0,2] range to [-1,1]
	#if PATH_TRACER_USE_VOLUMETRIC_SHADING_MODEL_DIRECTIONAL
		Payload.Anisotropy = clamp(GetMaterialTranslucencyDirectionalLightingIntensity() - 1, -1, 1);
	#else
		Payload.Anisotropy = -1;
	#endif

		Payload.DiffuseColor = BaseColor;
		Payload.SpecularColor = 0;
		Payload.RoughnessData = 1;
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_VOLUMETRIC;
	}
	else
#elif MATERIAL_SHADINGMODEL_UNLIT
	if (ShadingModelID == SHADINGMODELID_UNLIT)
	{
		// Regular unlit material, make sure the volumetric component is zero'ed out
		Payload.DiffuseColor = 0;
		Payload.SpecularColor = 0;
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
	}
	else
#endif // MATERIAL_SHADINGMODEL_UNLIT


#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	if (ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		const float ClearCoat = GetMaterialCustomData0(PixelMaterialInputs);
		const float ClearCoatRoughness = GetMaterialCustomData1(PixelMaterialInputs);

	#if CLEAR_COAT_BOTTOM_NORMAL
		#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
			float3 BottomNormal = ClearCoatBottomNormal0(MaterialParameters);
			#if MATERIAL_TANGENTSPACENORMAL
		    BottomNormal = normalize(TransformTangentVectorToWorld(MaterialParameters.TangentToWorld, BottomNormal));
			#endif
			Payload.SetBottomCoatNormal(BottomNormal);
		#else
			Payload.SetBottomCoatNormal(Payload.WorldNormal);
		#endif
	#endif
		Payload.RoughnessData.y = ClearCoatRoughness; // adjusted later
		Payload.RoughnessData.z = ClearCoat;
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_CLEARCOAT;
	}
	else
#endif // MATERIAL_SHADINGMODEL_CLEAR_COAT

#if MATERIAL_SHADINGMODEL_CLOTH
	if (ShadingModelID == SHADINGMODELID_CLOTH)
	{
		const float3 ClothColor = GetMaterialSubsurfaceData(PixelMaterialInputs).rgb;
		const float Fuzz = saturate(GetMaterialCustomData0(PixelMaterialInputs));
		// the default substrate model includes a fuzz layer, so we can do it this way
		// Additionally, tune the look like in SubstrateConvertLegacyMaterialDynamic
		Payload.SpecularColor *= 1 - Fuzz;
		Payload.FuzzColor = ClothColor;
		Payload.FuzzAmount = Fuzz;
		Payload.FuzzRoughness = 0.5 * Roughness + 0.5; // (adjust range because the legacy model has a very different response)
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
	}
	else
#endif // MATERIAL_SHADINGMODEL_CLOTH

#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	if (ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		const float3 SubsurfaceColor = GetMaterialSubsurfaceData(PixelMaterialInputs).rgb;
		Payload.SetFoliageTransmissionColor(SubsurfaceColor);
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_FOLIAGE;
	}
	else
#endif // MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE

#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
	if (ShadingModelID == SHADINGMODELID_SUBSURFACE ||
		ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		float3 SubsurfaceColor = GetMaterialSubsurfaceData(PixelMaterialInputs).rgb;

	#if HAVE_GetSubsurfaceMediumMaterialOutput0
		float3 SSSRadius = GetSubsurfaceMediumMaterialOutput0(MaterialParameters);
	#else
		// TODO: is this accurate enough? hard to measure exactly since the raster path uses a very different approach
		const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
		const float SSSRadius = (1 - Opacity) * 10.0; // simple formula, up to 10cm radius
	#endif

		

	#if HAVE_GetSubsurfaceMediumMaterialOutput1
		float PhaseG = clamp(GetSubsurfaceMediumMaterialOutput1(MaterialParameters), -0.99f, 0.99f);
	#else
		float PhaseG = 0.0;
	#endif

		AdjustPathTracingTwoSidedColorBalance(Payload.DiffuseColor, SubsurfaceColor);
		// default substrate slab includes SSS
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
		Payload.DiffuseColor += SubsurfaceColor; // no distinction between diffuse portion and SSS portion (could try using stochastic blending here instead?)
		Payload.MeanFreePath = SSSRadius;
		Payload.PhaseG = PhaseG;
	}
	else
#endif // MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN

#if MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE
	if (ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE)
	{
		const uint ProfileId = ExtractSubsurfaceProfileInt(GetMaterialSubsurfaceDataRaw(PixelMaterialInputs).a);

		// Decode dual roughness info
		float Lobe0Roughness = 0;
		float Lobe1Roughness = 0;
		float LobeMix = 0;

		// NOTE: we pass 1.0 for opacity because we don't really want to fade-out the dual roughness as opacity is lowered
		GetSubsurfaceProfileDualSpecular(ProfileId, Roughness, 1.0, Lobe0Roughness, Lobe1Roughness, LobeMix);

		Lobe0Roughness = AdjustMaterialRoughness(Lobe0Roughness, PackedPayload.GetPathRoughness());
		Lobe1Roughness = AdjustMaterialRoughness(Lobe1Roughness, PackedPayload.GetPathRoughness());

	#if HAVE_GetSubsurfaceMediumMaterialOutput0
		float3 SSSRadius = GetSubsurfaceMediumMaterialOutput0(MaterialParameters);
	#else
		float3 SSSRadius = 0;
		if (GetSubsurfaceProfileUseBurley(ProfileId))
		{
			// Decode SSS radius
			const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
			const float3 DiffuseColor = ComputeDiffuseAlbedo(BaseColor, Metallic);
			SSSRadius = DecodeSSSProfileRadius(ProfileId, DiffuseColor, Opacity);
		}
	#endif

	#if HAVE_GetSubsurfaceMediumMaterialOutput1
		float PhaseG = clamp(GetSubsurfaceMediumMaterialOutput1(MaterialParameters), -0.99f, 0.99f);
	#elif USE_SSS_PROFILE_ANISOTROPY
		// NOTE: this is not currently used
		float PhaseG = DecodeSSSProfileScatteringDistribution(ProfileId);
		SSSRadius *= 1 - PhaseG;
	#else
		float PhaseG = 0;
	#endif

		// substrate slab includes SSS and dual roughness already
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
		Payload.MeanFreePath = SSSRadius;
		Payload.PhaseG = PhaseG;
		Payload.RoughnessData = float3(Lobe0Roughness, Lobe1Roughness, LobeMix);
	}
	else
#endif // MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	if (ShadingModelID == SHADINGMODELID_SINGLELAYERWATER)
	{
		// For single layer water we always want to treat it as front facing when evaluating parameters 
		// as hitting back face is treated as being underwater and we get completely different parameters.
		FMaterialPixelParameters WaterMaterialParameters = MaterialParameters;
		WaterMaterialParameters.TwoSidedSign = 1; 

		const float3 ScatteringCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput0(WaterMaterialParameters)));
		const float3 AbsorptionCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput1(WaterMaterialParameters)));
		const float PhaseG = clamp(DFDemote(GetSingleLayerWaterMaterialOutput2(WaterMaterialParameters).x), -0.99f, 0.99f);

		const float3 WaterExtinction = ScatteringCoeff + AbsorptionCoeff;
		const float3 WaterAlbedo = select(WaterExtinction > 0.0, ScatteringCoeff / WaterExtinction, float3(0.0, 0.0, 0.0));

		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_SOLID_GLASS; // replace shading model to avoid extra cases in raygen
		Payload.MeanFreePath = rcp(WaterExtinction);
	    Payload.SetGlassAlbedo(WaterAlbedo);
		Payload.SetGlassPhase(PhaseG);
		Payload.Ior = DielectricF0ToIor(DielectricSpecularToF0(Specular));
		// we are using actual refraction, so don't apply any transparency
		Payload.TransparencyColor = 0.0;
		Payload.BSDFOpacity = 1.0;
	}
	else
#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	if (ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
	{
		const float3 TransmittanceColor = GetThinTranslucentMaterialOutput0(MaterialParameters);

		// The material is a blend of glass and default_lit response, model it stochastically
		if (Payload.BSDFOpacity < PackedPayload.GetStochasticSlabRand())
		{
			// Pick glass lobe - shade as glass (leave SHADINGMODELID_THIN_TRANSLUCENT on)
			Payload.Ior = Ior;
			Payload.SetTransmittanceColor(TransmittanceColor);
			Payload.DiffuseColor = 0;
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_THIN_GLASS;
		}
		else
		{
			// Pick solid lobe -- toggle back to default_lit response
			// Ior stays 0.0 since this mode does not include any refraction
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;
		}
		
		float SurfaceCoverage = GetThinTranslucentMaterialOutput1(MaterialParameters);
	#if PATH_TRACER_REFRACTION_ENABLED
		if (Ior > 0.0)
		{
			// In this mode we get rough refraction, so any transparency is only due to coverage (affecting both solid and glass lobes)
			Payload.TransparencyColor = 1 - SurfaceCoverage;
		}
		else
	#endif
		{
			// In this mode we just get straight transparency
			const float3 V = WorldRayDirection();
			const float3 N = normalize(Payload.WorldNormal);
			const float VoN = abs(dot(V, N));
			const float  F0 = F0RGBToF0(ComputeF0(Specular, BaseColor, Metallic));
			// This is the portion of transparency due to glass. We always compute it because we don't need to model the amount
			// of transparency stochastically.
			/// BSDFOpacity here still represents the blend between solid and glass behaviors.
			Payload.TransparencyColor = (1 - Payload.BSDFOpacity) * ComputeThinSlabWeights(TransmittanceColor, VoN, 0.0, F0).Transmitted;
			// Now account for overall coverage (allows the combination of solid/glass to become transparent)
			Payload.TransparencyColor = lerp(1.0, Payload.TransparencyColor, SurfaceCoverage);
		}
		// Now we can replace the BSDFOpacity with coverage
		Payload.BSDFOpacity = SurfaceCoverage;
	}
	else
#endif // MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT

#if MATERIAL_SHADINGMODEL_HAIR
	if (ShadingModelID == SHADINGMODELID_HAIR)
	{
		Payload.SetBaseColor(BaseColor);
		Payload.SetHairLongitudinalRoughness(AdjustMaterialRoughness(Roughness, PackedPayload.GetPathRoughness()));
		Payload.SetHairAzimuthalRoughness(Metallic); // SUBSTRATE_TODO: Map from scatter param to azimuthal roughness here instead of in the raygen
		Payload.SetHairSpecular(Specular);

#if HAIR_STRAND_MESH_FACTORY
		Payload.SetHairPrimitiveUV(MaterialParameters.HairPrimitiveUV);
#else
		Payload.SetHairPrimitiveUV(0.5);
#endif
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_HAIR;
	}
	else
#endif // MATERIAL_SHADINGMODEL_HAIR

#if MATERIAL_SHADINGMODEL_EYE
	if (ShadingModelID == SHADINGMODELID_EYE)
	{
		// This is all based on logic from ShadingModelsMaterial.ush
		const float IrisMask     = saturate(GetMaterialCustomData0(PixelMaterialInputs));
		const float IrisDistance = saturate(GetMaterialCustomData1(PixelMaterialInputs));
		Payload.SetEyeIrisMask(IrisMask);
		Payload.SpecularColor = ComputeF0(Specular, BaseColor, 0);

	#if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
		// Blend in the negative intersection normal to create some concavity
		// Not great as it ties the concavity to the convexity of the cornea surface
		// No good justification for that. On the other hand, if we're just looking to
		// introduce some concavity, this does the job.
		const float3 PlaneNormal = normalize(GetTangentOutput0(MaterialParameters));
		const float3 CausticNormal = normalize(lerp(PlaneNormal, -Payload.WorldNormal, IrisMask * IrisDistance));
		Payload.SetEyeCausticNormal(CausticNormal);
		// NOTE: calling AdjustShadingNormal on the custom normal does not seem necessary since this is only used for a diffuse calculation
	#else
		const float3 PlaneNormal = Payload.WorldNormal;
		Payload.SetEyeCausticNormal(Payload.WorldNormal);
	#endif

	#if IRIS_NORMAL // on
		// Match BasePassPixelShader logic, Specular part of g-buffer is re-use, so hard code the term
		Payload.SpecularColor = DielectricSpecularToF0(0.25);

		#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
			float3 IrisNormal = normalize( ClearCoatBottomNormal0(MaterialParameters) );
			#if MATERIAL_TANGENTSPACENORMAL
				IrisNormal = normalize( TransformTangentVectorToWorld( MaterialParameters.TangentToWorld, IrisNormal ) );
			#endif
		#else
			float3 IrisNormal = PlaneNormal;
		#endif
		Payload.SetEyeIrisNormal(IrisNormal);
	#else // IRIS_NORMAL = off
		Payload.SetEyeIrisNormal(PlaneNormal);
	#endif

		
	#if HAVE_GetSubsurfaceMediumMaterialOutput0
		float3 SSSRadius = GetSubsurfaceMediumMaterialOutput0(MaterialParameters);
	#else
		float3 SSSRadius = 0;
		const uint ProfileId = ExtractSubsurfaceProfileInt(GetMaterialSubsurfaceDataRaw(PixelMaterialInputs).a);
		if (GetSubsurfaceProfileUseBurley(ProfileId))
		{
			// Decode SSS radius
			const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
			const float3 DiffuseColor = ComputeDiffuseAlbedo(BaseColor, Metallic);
			SSSRadius = DecodeSSSProfileRadius(ProfileId, DiffuseColor, Opacity);
		}
	#endif

	#if HAVE_GetSubsurfaceMediumMaterialOutput1
		float PhaseG = clamp(GetSubsurfaceMediumMaterialOutput1(MaterialParameters), -0.99f, 0.99f);
	#else
		float PhaseG = 0;
	#endif

		Payload.MeanFreePath = SSSRadius;
		Payload.PhaseG = PhaseG;
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_EYE;
	}
	else
#endif // MATERIAL_SHADINGMODEL_EYE

#if MATERIAL_SHADINGMODEL_DEFAULT_LIT
	if (ShadingModelID == SHADINGMODELID_DEFAULT_LIT)
	{
		Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_BASIC;

	#if PATH_TRACER_REFRACTION_ENABLED
		// only allow refraction for default lit materials since we need space for the absorption amount
		if (Ior > 0.0)
		{
			// Our material is a blend between glass and default_lit -- use stochastic blending between the two
			Payload.TransparencyColor = 0.0;
			if (PackedPayload.GetStochasticSlabRand() < Payload.BSDFOpacity)
			{
				// solid case - keep default_lit shading model
			}
			else
			{
				// glass
				// we are using actual refraction, so disable transparency
				Payload.Ior = Ior;
				Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_SOLID_GLASS;

			#if HAVE_GetAbsorptionMediumMaterialOutput0
				float3 Extinction = PathTracingGlassTransmittanceToExtinction(GetAbsorptionMediumMaterialOutput0(MaterialParameters));
			#else
				// Make the glass totally clear if no custom medium is set
				float Extinction = 0;
			#endif

				Payload.MeanFreePath = rcp(Extinction);
				Payload.SetGlassAlbedo(0.0);
			}
			Payload.BSDFOpacity = 1.0; // cancels out with the probability of selecting the lobe
		}
	#endif
	}
	else
#endif // MATERIAL_SHADINGMODEL_DEFAULT_LIT
	{
		// terminal case for the conditionals above
	}

	// adjust after everything else (because SSS profile case needs to decode the dual spec info from the unmodified roughness)
	Payload.RoughnessData.x = AdjustMaterialRoughness(Payload.RoughnessData.x, PackedPayload.GetPathRoughness());
	Payload.RoughnessData.y = AdjustMaterialRoughness(Payload.RoughnessData.y, PackedPayload.GetPathRoughness());

#endif // SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

#if MATERIAL_TWOSIDED == 0
	if (HitKind() == HIT_KIND_TRIANGLE_BACK_FACE)
	{
		// when viewing the surface from "inside", don't include emission
		Payload.Radiance = 0;
	}
#endif

	// Only opaque surfaces can receive decals. This is both for compatibility with raster and improving performance when we have stacks of transparent surfaces
#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	if ((PrimitiveFlags & PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER) != 0)
	{
#if USE_DBUFFER
		Payload.SetDecalReceiver(MATERIALDECALRESPONSEMASK);
#	if MATERIAL_USES_DECAL_LOOKUP
		Payload.SetUseDBufferLookup();
#	endif
#else
		Payload.SetDecalReceiver(0x07);
#endif
	}
#endif

#if MATERIALBLENDING_ALPHAHOLDOUT
	// the material is already a holdout, no need to do anything else
#else
	if (PackedPayload.IsCameraRay())
	{
		if ((PrimitiveFlags & PRIMITIVE_SCENE_DATA_FLAG_HOLDOUT) != 0 && ResolvedView.bPrimitiveAlphaHoldoutEnabled)
		{
			if (Payload.Ior != 0.0)
			{
				// If the surface is marked as refractive then treat it as opaque for holdout
				Payload.BSDFOpacity = 1.0;
			}
			else
			{
				// keep the opacity from the material, it might indicate partial transparency
				// so we should get a partial holdout
			}

			Payload.SetHoldout();
			// zero-out the material contribution so it renders black
			Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
			Payload.Radiance = 0;
			Payload.SetBaseColor(0.0);
		}
	}
#endif

#if USES_WORLD_POSITION_OFFSET
	// WPO moves the position which invalidates the normal. This means the only way to compute a correctly oriented normal is for the shader to orient it properly
	// The path tracer normally keeps the WorldSmoothNormal and WorldNormal separately, but this is not possible in this case. To avoid issues from having these two
	// normals being un-aligned with each other, simply copy WorldNormal to WorldSmoothNormal when WPO is in use.
	// See repro in UE-343656
	#if ALWAYS_EVALUATE_WORLD_POSITION_OFFSET
		// always handle WPO
	#else
		// only if the primitive is using it
	if ((PrimitiveFlags & PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET) != 0u)
	#endif
	{
		Payload.WorldSmoothNormal = Payload.WorldNormal;
	}
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	if (PackedPayload.IsCameraRay())
	{
		// Virtual texturing feedback logic (camera rays only for now)
		FinalizeVirtualTextureFeedback(
			MaterialParameters.VirtualTextureFeedback,
			MaterialParameters.SvPosition,
			View.VTFeedbackBuffer
		);
	}
#endif

    // Support for debug modes that require information only avaiable inside CHS
    int DebugInfo = PackedPayload.GetFlags() & PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_INFO_MASK;
    if (DebugInfo)
    {
        Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT;
        Payload.SetBaseColor(0.0);
        uint PayloadTriangleID = PrimitiveIndex();
        uint PayloadInstanceID = InstanceID();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
        uint PayloadClusterID = GetClusterID();
#else
        uint PayloadClusterID = 0;
#endif
        uint Bits = StrongIntegerHash(PayloadInstanceID);
        if (DebugInfo == PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_CLUSTER_ID)
        {
            Bits += StrongIntegerHash(PayloadClusterID);
        }
        if (DebugInfo == PATH_TRACING_PAYLOAD_INPUT_FLAG_DEBUG_TRIANGLE_ID)
        {
            Bits += StrongIntegerHash(PayloadTriangleID);
        }
        Payload.Radiance = ((Bits >> int3(0, 10, 20)) & 1023) * (1.0f / 1023);
    }

	// Output depth for solid, masked 
	// or when `output depth and velocity' is set and Transparency is below the clip value to match the behavior of the rasterizer path. 
	// If refraction is enabled, we always output depth (but still check against the transparency to support glass with partial coverage
#if (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED)
	Payload.SetOutputDepth();
#elif ((TRANSLUCENT_WRITING_VELOCITY || PATH_TRACER_REFRACTION_ENABLED) && \
	(MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_ALPHAHOLDOUT))
	float TransparencyLimit = 1.0 - GetMaterialOpacityMaskClipValue();
	if (any(Payload.TransparencyColor < TransparencyLimit))
	{
		Payload.SetOutputDepth();
	}
#endif

	PackedPayload = PackPathTracingPayload(Payload);
}

#if USE_MATERIAL_ANY_HIT_SHADER

RAY_TRACING_ENTRY_ANY_HIT(PathTracingMaterialAHS,
	FPackedPathTracingPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
#if MATERIALBLENDING_MODULATE || (MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK) || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT
	if (!PackedPayload.IsVisibilityRay())
	{
		if (PackedPayload.GetFlags() & PATH_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT)
		{
			IgnoreHit();
		}
		// not a shadow ray -- don't need to run the AHS logic
		return;
	}
	if (PackedPayload.CheckDuplicateAHS(RayTCurrent()))
	{
		// We just processed this exact hit, don't double-count it
		IgnoreHit();
		return;
	}
#endif
	// This is the only case which actually needs to run the full material
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

	CurrentPayloadInputFlags = PackedPayload.GetFlags();
	PathTracerSceneDepth = (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_HAS_SCENE_DEPTH) ? PackedPayload.GetSceneDepth() : SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	// this is a newer codepath that is both more flexible and allows for more direct calculation compared to the other codepath
	// TODO: implement such a method for all vertex factories
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), GetClusterID(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FVertexFactoryInterpolantsVSToPS Interpolants;
	float3 GeoNormal = 0;
	CalcInterpolants((FRayCone)0, Attributes, Interpolants, GeoNormal);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif

	FPixelMaterialInputs PixelMaterialInputs;

	const bool bIsFrontFace = IsFrontFaceHit();

	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);

		MaterialParameters.CameraVector = -WorldRayDirection();

		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}
#if MATERIALBLENDING_MASKED && !MATERIAL_DITHER_OPACITY_MASK
	// Regardless of payload flags -- we always apply this 
	if (GetMaterialMask(PixelMaterialInputs) < 0)
	{
		IgnoreHit();
	}
#endif

	// the following blend modes need to process shadows after having executed the material
#if MATERIALBLENDING_MODULATE || (MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK) || MATERIALBLENDING_ALPHACOMPOSITE || MATERIALBLENDING_TRANSLUCENT
	PackedPayload.UpdateDuplicateAHS(RayTCurrent());

	{ // TODO: Cleanup this indentation level

#if SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

#if MATERIAL_IS_SUBSTRATE 
		FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
		FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();

		FSubstrateIntegrationSettings IntegrationSettings = InitSubstrateIntegrationSettings(
			false /* bForceFullyRough */,
			true  /* SubstrateStruct.bRoughDiffuse */,
			-1    /* SubstrateStruct.PeelLayersAboveDepth */,
			true  /* SubstrateStruct.bRoughnessTracking */
		);

		float TotalCoverage = 1;
		float3 TotalTransmittancePreCoverage = 0;
		const float3 V_World = MaterialParameters.CameraVector;
		SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V_World, IntegrationSettings, TotalCoverage, TotalTransmittancePreCoverage);
#if MATERIALBLENDING_TRANSLUCENT && (PATH_TRACER_REFRACTION_ENABLED || MATERIAL_ISTHINSURFACE)

		// SUBSTRATE_TODO: refraction handling
		float PathRoughness = PackedPayload.GetPathRoughness();

		// SUBSTRATE_TODO: This does not match the logic in DistortAccumulatePS.usf
		//  The idea here is to take F0 from the bottom refraction layers since this matches most closely what the refraction logic below does (only bottom slabs can refract).
		//  The distortion approximation on the raster side instead averages F0 across layers which may produce different results.
		//  In both cases - we get a potentially strange answer when there is more than one bottom slab that produces refraction

		// SUBSTRATE_TODO: How should we handle MFP when there are multiple bottom slabs?
		float F0 = 0.0f;
		float3 RoughnessData = 0.0;
		float NoV = 0.0f;
		float3 LocalSigmaT = 0.0f;
		float Ior = 1.0;
		float RefractMix  = 0.0;
		Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
		{
#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]
			if (BSDF.bIsBottom)
			{
				// SUBSTRATE_TODO: Improve this to correctly deal with multiple bottom slabs (in the case of non-parameter blended horizontal blends)
				float3 N = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
				NoV = dot(N, V_World);
				switch (BSDF_GETTYPE(BSDF))
				{
					case SUBSTRATE_BSDF_TYPE_SLAB:
					{
						// SUBSTRATE_TODO: Also consider the MFP to see how much this slab is contributing to refraction ....
						F0 = F0RGBToF0(SubstrateGetBSDFSpecularF0(BSDF));
						RoughnessData.x = SLAB_ROUGHNESS(BSDF);
						if (BSDF_GETHASHAZINESS(BSDF))
						{
							// NOTE: It should be safe to assume that the clearcoat case cannot happen here as it is only triggered by legacy clearcoat materials which are unlikely to have refraction
							const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
							RoughnessData.y = Haziness.Roughness;
							RoughnessData.z = Haziness.Weight;
						}
					#if PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE
						switch (BSDF_GETSSSTYPE(BSDF))
						{
							case SSS_TYPE_DIFFUSION:
							case SSS_TYPE_SIMPLEVOLUME:
							{
								// only non-SSS slabs 
								float3 MeanFreePath = SLAB_SSSMFP(BSDF);
								LocalSigmaT = rcp(max(MeanFreePath, 0.0001));
							#if PATH_TRACER_REFRACTION_ENABLED
								Ior = GetRefractionIor(PixelMaterialInputs, F0);
							#endif
							#if MATERIAL_ISTHINSURFACE
								float3 Transmittance = MeanFreePathToTransmittance(MeanFreePath * CENTIMETER_TO_METER, SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
								RefractMix = saturate(Luminance(Transmittance));
							#else
								// blend towards Opaque SSS in the range of [0cm, 1cm]
								RefractMix = saturate(max3(MeanFreePath.x, MeanFreePath.y, MeanFreePath.z));
							#endif
								break;
							}
							default:
							{
								// All other SSS types (including none) do not product any transmission and should be treated as opaque
								break;
							}
						}
						break;
					#endif // PATH_TRACER_USE_SUBSTRATE_GLASS_LOBE
					}
					case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
					{
						RoughnessData.x = SLW_ROUGHNESS(BSDF);
						RoughnessData.yz = 0;
#if SUBSTRATE_INLINE_SINGLELAYERWATER
						LocalSigmaT = SLW_WATEREXTINCTION(BSDF);
#endif
						float F0 = DielectricSpecularToF0(SLW_SPECULAR(BSDF));
						Ior = DielectricF0ToIor(F0);
						RefractMix = 1;
						break;
					}
					case SUBSTRATE_BSDF_TYPE_TOON:
					{
						// No refraction
						break;
					}
				}
			}
#undef BSDF
		}


#if MATERIAL_ISTHINSURFACE
		// Thin Glass Fake Caustics

		// SUBSTRATE_TODO: Revisit this -- TotalTransmittancePreCoverage should probably account for this in the thin case
		float3 Transmission = exp(-LocalSigmaT * SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM);
		float3 Tr = RefractMix * ComputeThinSlabWeights(Transmission, NoV, Ior, F0).Transmitted;
#else
		HLSL_STATIC_ASSERT(PATH_TRACER_REFRACTION_ENABLED == 1, "Expect refraction to be true here");
		// Solid Glass Fake Caustics
		float Tr = RefractMix * FakeCaustics(F0, Ior, NoV);
		// Account for transmission 
		PackedPayload.SetTau(PackedPayload.GetTau() + TotalCoverage * RefractMix * ComputeOpticalDepthForHit(LocalSigmaT, bIsFrontFace));
#endif

		// If material is doing refraction, apply an extra factor to account for ApproximateCaustics
#if PATH_TRACER_REFRACTION_ENABLED
		if (Ior > 0.0)
		{
			Tr *= lerp(FakeCausticsPathFactor(RoughnessData.x, PathRoughness),
			           FakeCausticsPathFactor(RoughnessData.y, PathRoughness), RoughnessData.z);
		}
#endif
		const float3 Transparency = saturate(lerp(1.0, Tr, TotalCoverage));

#else // MATERIALBLENDING_TRANSLUCENT && (PATH_TRACER_REFRACTION_ENABLED || MATERIAL_ISTHINSURFACE)
		// All other blend modes, do simple transparency handling
		const float3 Transparency = saturate(lerp(1.0, TotalTransmittancePreCoverage, TotalCoverage));
#endif

#else // MATERIAL_IS_SUBSTRATE

		// TODO: What cases (if any) end up here?
		const float3 Transparency = 0.0;
#endif // MATERIAL_IS_SUBSTRATE

#else // SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

#if MATERIALBLENDING_MODULATE
		const float3 Transparency = GetMaterialEmissive(PixelMaterialInputs);
#elif MATERIALBLENDING_ALPHACOMPOSITE
		const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
		const float Transparency = 1 - Opacity;
#elif MATERIALBLENDING_MASKED && MATERIAL_DITHER_OPACITY_MASK
		// See MATERIAL_DITHER_OPACITY_MASK comment below
		const float Opacity = saturate(GetMaterialMaskInputRaw(PixelMaterialInputs));
		const float Transparency = 1 - Opacity;
#elif MATERIALBLENDING_TRANSLUCENT
		const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
		// We can only get colored shadows for thin translucent (solid translucent case happens via absorption only)
		float3 Transparency = 1 - Opacity;
#else
		float Transparency = 1 - Opacity;
#endif
		uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
		if (ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
		{
			float3 Transmission = GetThinTranslucentMaterialOutput0(MaterialParameters);
#if PATH_TRACER_REFRACTION_ENABLED
			float Ior = GetRefractionIor(PixelMaterialInputs);
#else
			float Ior = 0.0;
#endif
			float3 V = WorldRayDirection();
			float3 N = normalize(MaterialParameters.WorldNormal);
			float VoN = abs(dot(V, N));
			if (Opacity < 1.0)
			{
				float PathRoughness = PackedPayload.GetPathRoughness();
				float Roughness = GetMaterialRoughness(PixelMaterialInputs);
				if (Ior > 0.0 && PathRoughness <= Roughness)
				{
					// not using fast caustic approximation - treat as opaque
					PackedPayload.SetRayThroughput(0.0);
					return;
				}
				// compute transmission through the slab (fresnel + absorption)
				float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
				float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
				float  Specular = GetMaterialSpecular(PixelMaterialInputs);
				float  F0 = F0RGBToF0(ComputeF0(Specular, BaseColor, Metallic));
				Transparency *= ComputeThinSlabWeights(Transmission, VoN, Ior, F0).Transmitted;

				// fake caustic approximation (see comments below)
#if PATH_TRACER_REFRACTION_ENABLED
				if (Ior > 0.0)
				{
					Transparency *= FakeCausticsPathFactor(Roughness, PathRoughness);
				}
#endif
			}

			// Fade out the Transparency as a function of the SurfaceCoverage.
			const float SurfaceCoverage = GetThinTranslucentMaterialOutput1(MaterialParameters);
			Transparency = lerp(1.0.xxx, Transparency, SurfaceCoverage);
		}
		else
#endif // MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
#if MATERIAL_SHADINGMODEL_DEFAULT_LIT && PATH_TRACER_REFRACTION_ENABLED
		if (ShadingModelID == SHADINGMODELID_DEFAULT_LIT)
		{
			// Is refraction enabled?
			float Ior = GetRefractionIor(PixelMaterialInputs);
			if (Transparency > 0 && Ior > 0.0)
			{
				// current hit has some refraction
				float PathRoughness = PackedPayload.GetPathRoughness();
				float Roughness = GetMaterialRoughness(PixelMaterialInputs);

				float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
				float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
				float  Specular = GetMaterialSpecular(PixelMaterialInputs);
				float  F0 = F0RGBToF0(ComputeF0(Specular, BaseColor, Metallic));

				float3 N = normalize(MaterialParameters.WorldNormal);
				float NoV = dot(WorldRayDirection(), N);

				Transparency *= FakeCaustics(F0, Ior, NoV) * FakeCausticsPathFactor(Roughness, PathRoughness);

#if HAVE_GetAbsorptionMediumMaterialOutput0
				// Does the material have any kind of volumetric absorption to apply?
				float3 TransmittanceColor = GetAbsorptionMediumMaterialOutput0(MaterialParameters);
				float3 LocalSigmaT = PathTracingGlassTransmittanceToExtinction(TransmittanceColor);
				
				PackedPayload.SetTau(PackedPayload.GetTau() + ComputeOpticalDepthForHit(LocalSigmaT, bIsFrontFace));
#endif // HAVE_GetAbsorptionMediumMaterialOutput0
			}
		}
		else
#endif // MATERIAL_SHADINGMODEL_DEFAULT_LIT
		{
			// base case for shadingmodel if/else
		}
#else // MATERIALBLENDING_*
#error Unhandled blending mode!
#endif

#endif // SUBSTRATE_ENABLED && !USE_LEGACY_CODEPATH_FOR_SUBSTRATE

		// Update the ray throughput (it is packed simply into the payload since we don't need to carry any other information across hits)
		float3 RayThroughput = PackedPayload.GetRayThroughput();
		RayThroughput *= Transparency;
		PackedPayload.SetRayThroughput(RayThroughput);
		if (any(RayThroughput > 0))
		{
			// keep tracing if we still have some energy left
			IgnoreHit();
		}
	}
#endif
}

#endif // USE_MATERIAL_ANY_HIT_SHADER
