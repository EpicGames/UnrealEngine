// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathTracingCommon.ush"

RAY_TRACING_ENTRY_MISS(PathTracingDefaultMS, FPackedPathTracingPayload, Payload)
{
	Payload.SetMiss();
	Payload.SetFlag(0); // clear input flags
}
