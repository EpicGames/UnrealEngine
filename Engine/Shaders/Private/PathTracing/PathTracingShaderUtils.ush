// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file contains utility methods that can be used by path tracer specific material shaders (such as the CHS/AHS pairs, light miss shaders, etc ...)

#ifndef PATH_TRACING
#error This header is only for use within path tracing shaders
#endif

#include "PathTracingCommon.ush"

bool GetPathTracingQualitySwitch()
{
#if SIMPLIFIED_MATERIAL_SHADER
	// GPU Lightmass does not take the pathtracing branch as it has a different node dedicated to it
	return false;
#else
	return true;
#endif
}

// Use a global variable for convenience. This must be set prior to running the generated material code.
static int CurrentPayloadInputFlags = 0;
bool GetPathTracingIsShadow()
{
	return (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_SHADOW;
}

bool GetPathTracingIsIndirectDiffuse()
{
	return (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_DIFFUSE;
}

bool GetPathTracingIsIndirectSpecular()
{
	return (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_SPECULAR;
}

bool GetPathTracingIsIndirectVolume()
{
	return (CurrentPayloadInputFlags & PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_MASK) == PATH_TRACING_PAYLOAD_INPUT_FLAG_RAY_TYPE_INDIRECT_VOLUME;
}

#if RAYHITGROUPSHADER
// In the CHS/AHS case, also define these (which will be missing from Common.ush)
bool GetShadowReplaceState()
{
	return GetPathTracingIsShadow();
}

float IsShadowDepthShader()
{
	return GetShadowReplaceState() ? 1.0f : 0.0f;
}
#endif
