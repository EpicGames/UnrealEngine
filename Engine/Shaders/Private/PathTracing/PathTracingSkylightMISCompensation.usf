// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

Texture2D<float> SkylightTexturePdfAverage;
RWTexture2D<float4> SkylightTextureOutput;
RWTexture2D<float> SkylightTexturePdf;


[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingSkylightMISCompensationCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	// anything smaller than the average value will be well handled by BSDF samping when MIS is enabled
	// so subtract it from the base value
	// A more complete theoretical derivation is given in:
	// [Karlík et al, 2019] "MIS Compensation: Optimizing Sampling Techniques in Multiple Importance Sampling"
	// https://cgg.mff.cuni.cz/~jaroslav/papers/2019-mis-compensation/index.html

	float AverageValue = SkylightTexturePdfAverage.Load(0); // bound to 1x1 mip level
	float Prob = max(SkylightTexturePdf[DispatchThreadId] - AverageValue, 0.0);
	SkylightTextureOutput[DispatchThreadId] = float4(SkylightTextureOutput[DispatchThreadId].xyz, Prob);
	SkylightTexturePdf[DispatchThreadId] = Prob;
}
