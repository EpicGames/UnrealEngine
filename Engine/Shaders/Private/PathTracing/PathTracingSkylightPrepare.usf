// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../MonteCarlo.ush"

TextureCube SkyLightCubemap0;
TextureCube SkyLightCubemap1;
SamplerState SkyLightCubemapSampler0;
SamplerState SkyLightCubemapSampler1;
float SkylightBlendFactor;
float SkylightInvResolution;

RWTexture2D<float4> SkylightTextureOutput;
RWTexture2D<float> SkylightTexturePdf;
float3 SkyColor;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingSkylightPrepareCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float2 UV = (DispatchThreadId + 0.5) * SkylightInvResolution;
	// Highest resolution level -- capture the actual cube map(s) data
	float3 ReflectionVector = EquiAreaSphericalMapping(UV).zxy;
	float3 Reflection = TextureCubeSample(SkyLightCubemap0, SkyLightCubemapSampler0, ReflectionVector).rgb;
	if (SkylightBlendFactor > 0)
	{
		float3 Reflection1 = TextureCubeSample(SkyLightCubemap1, SkyLightCubemapSampler1, ReflectionVector).rgb;

		Reflection = lerp(Reflection, Reflection1, SkylightBlendFactor);
	}

	float3 Pixel = max(Reflection * SkyColor, 0.0);
	float Prob = max(Pixel.r, max(Pixel.g, Pixel.b));
	// record color and probability density (as the max of each component)
	SkylightTextureOutput[DispatchThreadId] = float4(Pixel, Prob);
	SkylightTexturePdf[DispatchThreadId] = Prob;
}
