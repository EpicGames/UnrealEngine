// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
PathTracingSpatialTemporalDenoising.usf: Spatial temporal denoising for path tracing
===============================================================================================*/
#pragma once

#include "/Engine/Public/Platform.ush"
#include "../ScreenPass.ush"
#include "../../Private/Common.ush"
#include "../TextureSampling.ush"
#include "../ColorDifference.ush"

#ifndef THREAD_SIZE
#define THREAD_SIZE						8
#define THREAD_SIZE_X					THREAD_SIZE
#define THREAD_SIZE_Y					THREAD_SIZE
#endif

#ifndef TEMPORAL_REPROJECTION_ALIGN
#define TEMPORAL_REPROJECTION_ALIGN		0
#endif

#ifndef TEMPORAL_REPROJECTION_BLUR
#define TEMPORAL_REPROJECTION_BLUR		0
#define DIRECT_COPY						0
#endif

#ifndef TEMPORAL_REPROJECTION_MERGE
#define TEMPORAL_REPROJECTION_MERGE		0
#endif

#ifndef TEMPORAL_REPROJECTION_RESOLVE
#define TEMPORAL_REPROJECTION_RESOLVE	0
#endif

#ifndef TEMPORAL_FEATURE_FUSION
#define TEMPORAL_FEATURE_FUSION			0
#endif

// Variance Type
#define RADIANCE_MULTI_CHANNEL					0
#define RADIANCE_ALBEDO_NORMAL_SINGLECHANNEL	1

#ifndef TEMPORAL_PREPASS
#define TEMPORAL_PREPASS				0
#define VARIANCE_TYPE					RADIANCE_MULTI_CHANNEL
#endif

#define PREPASS_PHASE_INIT				0
#define PREPASS_PHASE_UPDATE			1

#ifndef RANKED_LUMINANCE_VARIANCE
#define RANKED_LUMINANCE_VARIANCE		0
#endif

#ifndef SPATIAL_DENOISING
#define SPATIAL_DENOISING				0
#endif

#ifndef PREPASS_GENERATE_TEXTURE
#define PREPASS_GENERATE_TEXTURE		0
#endif

#ifndef VISUALIZE_MOTIONVECTOR
#define VISUALIZE_MOTIONVECTOR			0
#endif

#ifndef MOTION_VECTOR_SUBTRACT
#define MOTION_VECTOR_SUBTRACT			0
#endif

#ifndef TEMPORAL_HIGHFREQ_REJECT
#define TEMPORAL_HIGHFREQ_REJECT		0
#endif

#ifndef VISUALIZE_WARPING
#define VISUALIZE_WARPING				0
#endif 

#ifndef TOTAL_VARIATION
#define TOTAL_VARIATION					0
#endif

#ifndef PREPROCESS_BUFFER
#define PREPROCESS_BUFFER				0
#endif

// Define the distance metrics to use
#define METRICS_LUMINANCE				0
#define METRICS_EUCLIDEAN				1
#define METRICS_PERCEPTION				2

#ifndef DISTANCE_METRICS
#define DISTANCE_METRICS METRICS_LUMINANCE
#endif

// Gamma correction improves the reprojection of euclidean and luminance
// based distance metrics
#define APPLY_SIMPLIFIED_GAMMA (DISTANCE_METRICS == METRICS_EUCLIDEAN || \
								DISTANCE_METRICS == METRICS_LUMINANCE)

#ifndef K_NUM_OF_TEXTURES_PER_PASS
#define K_NUM_OF_TEXTURES_PER_PASS		7
#endif

#ifndef K_NUM_OF_SHIFTS_PER_SLICE
#define K_NUM_OF_SHIFTS_PER_SLICE		4
#endif

// the mip difference is 2 instead of one to make use of coarser mips
#define MIP_DIFF_DELTA 2

// The motion vector estimation algorithm is inspired from
// Hanika, J., Tessari, L., & Dachsbacher, C. (2021). 
// Fast Temporal Reprojection without Motion Vectors. Journal of Computer Graphics Techniques Vol, 10(3).

// Distance type (L1, L2, logL1, logL1^C)
#define DISTANCE_L1						0
#define DISTANCE_L2						1
#define DISTANCE_LOG_L1					2
#define DISTANCE_LOG_REGULARIZED_L1		3
#define DISTANCE_TYPE					DISTANCE_LOG_REGULARIZED_L1

SCREEN_PASS_TEXTURE_VIEWPORT(TargetViewport)

struct FPixelOffset
{
	float2		xy;
	float		Dist;
	float		TV;
};

struct FPixelMaterialLightingFingerprint
{
	float4 Mean;
	float4 Var;
};

void Max3Index(in float3 Value, out int MaxIndex)
{
	MaxIndex = (Value.x > Value.y) ? 0 : 1;
	MaxIndex = Value[MaxIndex] > Value.z ? MaxIndex : 2;
}

void Max3(in float3 Value, out float MaxValue, out int MaxIndex)
{
	MaxIndex = (Value.x > Value.y) ? 0 : 1;
	MaxIndex = Value[MaxIndex] > Value.z ? MaxIndex : 2;
	MaxValue = Value[MaxIndex];
}

void Min3Index(in float3 Value, out int MinIndex)
{
	MinIndex = (Value.x <= Value.y) ? 0 : 1;
	MinIndex = Value[MinIndex] <= Value.z ? MinIndex : 2;
}

void Min3(in float3 Value, out float MinValue, out int MinIndex)
{
	MinIndex = (Value.x <= Value.y) ? 0 : 1;
	MinIndex = Value[MinIndex] <= Value.z ? MinIndex : 2;
	MinValue = Value[MinIndex];
}

#if K_NUM_OF_TEXTURES_PER_PASS <= 7 && (TEMPORAL_REPROJECTION_ALIGN||TEMPORAL_REPROJECTION_MERGE)

static int2 Shifts[25] = {
	{-2,-2},{-1,-2},{0,-2},{1,-2},{2,-2},
	{-2,-1},{-1,-1},{0,-1},{1,-1},{2,-1},
	{-2, 0},{-1, 0},{0, 0},{1, 0},{2, 0},
	{-2, 1},{-1, 1},{0, 1},{1, 1},{2, 1},
	{-2, 2},{-1, 2},{0, 2},{1, 2},{2, 2},
};

RWTexture2D<float4> RWDistanceTextures_0;
RWTexture2D<float4> RWDistanceTextures_1;
RWTexture2D<float4> RWDistanceTextures_2;
RWTexture2D<float4> RWDistanceTextures_3;
RWTexture2D<float4> RWDistanceTextures_4;
RWTexture2D<float4> RWDistanceTextures_5;
RWTexture2D<float4> RWDistanceTextures_6;

Texture2D<float4> DistanceTextures_0;
Texture2D<float4> DistanceTextures_1;
Texture2D<float4> DistanceTextures_2;
Texture2D<float4> DistanceTextures_3;
Texture2D<float4> DistanceTextures_4;
Texture2D<float4> DistanceTextures_5;
Texture2D<float4> DistanceTextures_6;


void SaveOffsets(int2 Position, int BasePixelShift, float4 Value)
{

	if (BasePixelShift < 4)
	{
		RWDistanceTextures_0[Position] = Value;
	}
	else if (BasePixelShift < 8)
	{
		RWDistanceTextures_1[Position] = Value;
	}
	else if (BasePixelShift < 12)
	{
		RWDistanceTextures_2[Position] = Value;
	}
	else if (BasePixelShift < 16)
	{
		RWDistanceTextures_3[Position] = Value;
	}
	else if (BasePixelShift < 20)
	{
		RWDistanceTextures_4[Position] = Value;
	}
	else if (BasePixelShift < 24)
	{
		RWDistanceTextures_5[Position] = Value;
	}
	else if (BasePixelShift < 28)
	{
		RWDistanceTextures_6[Position] = Value;
	}
}

float GetOffsets(int2 Position, int2 Offset)
{
	Offset = clamp(Offset, int2(-2, -2), int2(2, 2));
	uint Index = Offset.x + Offset.y * 5 + 12;
	uint TextureIndexOffset = Index % 4;
	int2 ShiftedPosition = Position + Offset;
	//@TODO: range check
	float Ret = 0.0f;
	if (Index < 4)
	{
		Ret = DistanceTextures_0[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 8)
	{
		Ret = DistanceTextures_1[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 12)
	{
		Ret = DistanceTextures_2[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 16)
	{
		Ret = DistanceTextures_3[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 20)
	{
		Ret = DistanceTextures_4[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 24)
	{
		Ret = DistanceTextures_5[ShiftedPosition][TextureIndexOffset];
	}
	else if (Index < 28)
	{
		Ret = DistanceTextures_6[ShiftedPosition][TextureIndexOffset];
	}
	return Ret;
}

struct FQuadricFittingContext
{
	int2			Offset;	// integer shift of the minimal value
	float3x3	   Values;  // holding the surrounding distance to fit a bivariate polynomial.
};

float Min3(float3 value)
{
	return min(min(value.x, value.y), value.z);
}

float Min4(float4 value)
{
	return min(min(value.x, value.y), min(value.z, value.w));
}

struct FFindMinContext
{
	int TextureOffset;
	int Index;
	float Value;
};

void Min4(in float4 Value, out float MinValue, out int MinIndex)
{
	int MinIndexXy = (Value.x <= Value.y) ? 0 : 1;
	int MinIndexZw = (Value.z <= Value.w) ? 2 : 3;
	MinIndex = (Value[MinIndexXy] <= Value[MinIndexZw]) ? MinIndexXy : MinIndexZw;
	MinValue = Value[MinIndex];
}

void Max4(in float4 Value, out float MaxValue, out int MaxIndex)
{
	int MaxIndexXy = (Value.x < Value.y) ? 1 : 0;
	int MaxIndexZw = (Value.z < Value.w) ? 3 : 2;
	MaxIndex = (Value[MaxIndexXy] < Value[MaxIndexZw]) ? MaxIndexZw : MaxIndexXy;
	MaxValue = Value[MaxIndex];
}

void UpdateMin4Index(int TextureOffset, float4 Value, inout FFindMinContext Context)
{
	int MinIndex;
	float MinValue;
	Min4(Value, MinValue, MinIndex);

	if (MinValue < Context.Value)
	{
		Context.TextureOffset = TextureOffset;
		Context.Index = MinIndex;
		Context.Value = MinValue;
	}
}

#define ESTIMATE_SUBPIXELOFFSET SUBPIXEL_OFFSET

#if ESTIMATE_SUBPIXELOFFSET

// Ref: Section 4 of the Appendix of "Burst photography for high dynamic range and low-light imaging on mobile cameras"
// https://dl.acm.org/doi/10.1145/2980179.2980254, 2016
// solve a weighted least-squares problem with weight of
// 1 2 1
// 2 4 2
// 1 2 1

static const float FA11[9] = {	1.0f, -2.0f,  1.0f,
								2.0f, -4.0f,  2.0f,
								1.0f, -2.0f,  1.0f};

static const float FA22[9] = {	1.0f / 4.0f,  2.0f / 4.0f,  1.0f / 4.0f,
							   -2.0f / 4.0f, -4.0f / 4.0f, -2.0f / 4.0f,
								1.0f / 4.0f,  2.0f / 4.0f,  1.0f / 4.0f };


static const float FA12[9] = {	1.0f / 4.0f,  0.0f, -1.0f / 4.0f,
									   0.0f,  0.0f,         0.0f,
							   -1.0f / 4.0f,  0.0f,  1.0f / 4.0f }; // FA12 == FA21

static const float  FB1[9] = {  -1.0f / 8.0f,  0.0f,  1.0f / 8.0f,
							   -2.0f / 8.0f,  0.0f,  2.0f / 8.0f,
							   -1.0f / 8.0f,  0.0f,  1.0f / 8.0f };

static const float  FB2[9] = {  -1.0f / 8.0f, -2.0f / 8.0f, -1.0f / 8.0f,
							    0.0f,  0.0f,					   0.0f,
							    1.0f / 8.0f,  2.0f / 8.0f,  1.0f / 8.0f };

//static const float  FC[] = { -1.0f,  2.0f, -1.0f,
//								2.0f, 12.0f,  2.0f,
//							   -1.0f,  2.0f, -1.0f } / 16;

float Dot9(float A[9], float B[9])
{
	return	A[0] * B[0] + A[1] * B[1] + A[2] * B[2] +
			A[3] * B[3] + A[4] * B[4] + A[5] * B[5] +
			A[6] * B[6] + A[7] * B[7] + A[8] * B[8];
}

// D holds the 3x3 image distance. Return the subpixel offset 
// in range [-1, 1] x [-1, 1]
float2 EstimateSubpixelOffset(float3x3 D)
{
	// Step 1. Construct the constant coefficient A, b, c
	// for the bivariate quadratic function.

	float DSub[9];
	for (int j = -1; j <= 1; j++)
		for (int i = -1; i <= 1; i++)
			DSub[3 * (j + 1) + i + 1] = D[j + 1][i + 1];

	// force A positive semi-definite (PSD) with max(0, Aij) and reset non-diag
	// to zero if det is negative.
	// note: A12 == A21
	float4 A = { max(0.0f, Dot9(FA11, DSub)),			   Dot9(FA12, DSub),
				 0.0f						,	max(0.0f, Dot9(FA22, DSub))};

	float DetA = A[0] * A[3] - A[1] * A[1];
	
	if (DetA < 0.0f)
	{
		A[1] = 0;
		DetA = A[0] * A[3];
	}

	float2 b = { Dot9(FB1, DSub), 
				 Dot9(FB2, DSub)};

	//float c = Dot9(FC, DSub);

	// Get the sub-pixel offset
	// \mu = -A^{-1}b, 
	// the constant parameter s = c - \frac{\mu^{T}A\mu}{2} is ignored

	float2 SubpixelOffset = 0.0f;

	if (abs(DetA) > 1e-9)
	{
		float2 Mu = { -(A[3] * b[0] - A[1] * b[1]) / DetA,
					  -(A[0] * b[1] - A[1] * b[0]) / DetA };

		SubpixelOffset = lerp(0, Mu, length2(Mu) < 1);
	}
	
	return SubpixelOffset;
}
#endif

#define MAX_HEAP_SIZE 4

#if MAX_HEAP_SIZE != 4
	#error Support up to max heap size of 4 only
#endif

struct FMaxHeap
{
	float Heap[MAX_HEAP_SIZE];
	uint Index[MAX_HEAP_SIZE];
};

FMaxHeap CreateMaxHeap()
{
	FMaxHeap MaxHeap;
	
	UNROLL
	for (int i = 0; i < MAX_HEAP_SIZE; ++i)
	{
		MaxHeap.Heap[i] = MaxHalfFloat;
		MaxHeap.Index[i] = 0;
	}

	return MaxHeap;
}

float GetMax(in float Array[MAX_HEAP_SIZE])
{
	return Array[0];
}

void _Swap(inout float Array[MAX_HEAP_SIZE], uint i, uint j)
{
	float TmpValue = Array[i];
	Array[i] = Array[j];
	Array[j] = TmpValue;
}

void _Swap(inout uint Array[MAX_HEAP_SIZE], uint i, uint j)
{
	uint TmpValue = Array[i];
	Array[i] = Array[j];
	Array[j] = TmpValue;
}

void Insert(inout FMaxHeap MaxHeap, float Value, int Index)
{
	// Insert only when the value is smaller than the max value of the max heap
	if (Value < GetMax(MaxHeap.Heap))
	{
		// 1. replace the value at the head
		MaxHeap.Heap[0] = Value;
		MaxHeap.Index[0] = Index;
		
		// 2. Find max and swap to the head
		float MaxValue;
		int MaxIndex;
		Max4(float4(MaxHeap.Heap[0], MaxHeap.Heap[1], MaxHeap.Heap[2], MaxHeap.Heap[3]), MaxValue, MaxIndex);

		if (MaxIndex != 0)
		{
			_Swap(MaxHeap.Heap, 0, MaxIndex);
			_Swap(MaxHeap.Index, 0, MaxIndex);
		}
	}
}

void Insert(inout FMaxHeap MaxHeap, float4 Values, int4 Indices)
{
	UNROLL
	for (int i = 0; i < 4; ++i)
	{
		Insert(MaxHeap, Values[i], Indices[i]);
	}
}

void Sort4(float Values[MAX_HEAP_SIZE], inout uint4 Indices)
{
	uint Index[MAX_HEAP_SIZE] = {0, 1, 2, 3};
	// Optimal number of comparison is 5
	if (Values[Index[0]] < Values[Index[1]])
	{
		_Swap(Index, 0, 1);
	}

	if (Values[Index[2]] < Values[Index[3]])
	{
		_Swap(Index, 2, 3);
	}

	if (Values[Index[0]] < Values[Index[2]])
	{
		_Swap(Index, 0, 2);
	}

	if (Values[Index[1]] < Values[Index[3]])
	{
		_Swap(Index, 1, 3);
	}

	if (Values[Index[1]] < Values[Index[2]])
	{
		_Swap(Index, 1, 2);
	}
	Indices = uint4(Index[0], Index[1], Index[2], Index[3]);
}

// based on the target position, and the shift from the last mip, we fetch
FQuadricFittingContext GetQuadricFittingContext(int2 Position, int kth = 0)
{
	kth = clamp(kth, 0, MAX_HEAP_SIZE - 1);

	// get the minimal distance and the surrounding
	FFindMinContext Context = (FFindMinContext)0;
	
	BRANCH
	if (kth == 0)
	{
		Context.TextureOffset = 3;// starting to have (0,0) as the minimal value

		Min4(DistanceTextures_3[Position], Context.Value, Context.Index);

		UpdateMin4Index(1, DistanceTextures_1[Position], Context);
		UpdateMin4Index(2, DistanceTextures_2[Position], Context);
		UpdateMin4Index(4, DistanceTextures_4[Position], Context);
		UpdateMin4Index(0, DistanceTextures_0[Position], Context);
		UpdateMin4Index(5, DistanceTextures_5[Position], Context);

		int MinIndex = 0;
		float MinValue = DistanceTextures_6[Position].x;
		if (MinValue < Context.Value)
		{
			Context.TextureOffset = 6;
			Context.Value = MinValue;
			Context.Index = MinIndex;
		}
	}
	else
	{
		//TODO: assign the first distance texture values
		FMaxHeap MaxHeap = CreateMaxHeap();

		Insert(MaxHeap, DistanceTextures_3[Position], 3 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_1[Position], 1 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_2[Position], 2 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_4[Position], 4 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_0[Position], 0 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_5[Position], 5 * 4 + int4(0, 1, 2, 3));
		Insert(MaxHeap, DistanceTextures_6[Position].x, 6 * 4);

		uint4 Max4Index;
		Sort4(MaxHeap.Heap, Max4Index);

		Context.TextureOffset = Max4Index[kth] / 4;
		Context.Index = Max4Index[kth] % 4;
		Context.Value = GetMax(MaxHeap.Heap);

	}
	
	FQuadricFittingContext QuadContext;
	QuadContext.Offset = Shifts[Context.TextureOffset * 4 + Context.Index];


#if !ESTIMATE_SUBPIXELOFFSET
	QuadContext.Values = Context.Value;
#else
	int2 ShiftedPosition = Position + QuadContext.Offset;
	QuadContext.Values[0][0] = GetOffsets(ShiftedPosition, int2(-1, -1));
	QuadContext.Values[0][1] = GetOffsets(ShiftedPosition, int2( 0, -1));
	QuadContext.Values[0][2] = GetOffsets(ShiftedPosition, int2( 1, -1));
	QuadContext.Values[1][0] = GetOffsets(ShiftedPosition, int2(-1,  0));
	QuadContext.Values[1][1] = Context.Value;
	QuadContext.Values[1][2] = GetOffsets(ShiftedPosition, int2( 1,  0));
	QuadContext.Values[2][0] = GetOffsets(ShiftedPosition, int2(-1,  1));
	QuadContext.Values[2][1] = GetOffsets(ShiftedPosition, int2( 0,  1));
	QuadContext.Values[2][2] = GetOffsets(ShiftedPosition, int2( 1,  1));
#endif
	return QuadContext;
}

float GetTotalVariation(int2 Position, FQuadricFittingContext Context)
{
	int2 ShiftedPosition = Position + Context.Offset;
	float TotalVariation = 0.0f;
	for (int i = -2; i < 2; ++i)
	{
		for (int j = -2; j < 2; ++j)
		{
			float vij	= GetOffsets(ShiftedPosition, int2(		i,		j));
			float vi_1j = GetOffsets(ShiftedPosition, int2(	i + 1,		j));
			float vij_1 = GetOffsets(ShiftedPosition, int2(		i,	j + 1));

			TotalVariation += abs(vi_1j - vij) + abs(vij_1 - vij);
		}
	}

	for (int j = -2; j < 2; ++j)
	{
		float vij = GetOffsets(ShiftedPosition,		int2(2,		j));
		float vij_1 = GetOffsets(ShiftedPosition,	int2(2, j + 1));
		TotalVariation += abs(vij - vij_1);
	}

	for (int i = -2; i < 1; ++i)
	{
		float vij	= GetOffsets(ShiftedPosition, int2(		i,	2));
		float vi_1j = GetOffsets(ShiftedPosition, int2(	i + 1,	2));
		TotalVariation += abs(vij - vi_1j);
	}
	return TotalVariation;
}

FPixelOffset GetQuadricFittingOffset(FQuadricFittingContext Context)
{
	FPixelOffset Offset;

#if ESTIMATE_SUBPIXELOFFSET
	float2 SubpixelOffset = EstimateSubpixelOffset(Context.Values);
	Offset.xy = Context.Offset + SubpixelOffset;
#else
	Offset.xy = Context.Offset;
#endif
	
	Offset.Dist = Context.Values[1][1];
	return Offset;
}

#endif

int2 GetShiftedPosition(int2 Position, int2 Shift)
{
	return Position + Shift;
}

float2 GetBufferUVFromPosition(float2 Position, int MipLevel)
{
	return (Position + 0.5f) / ((uint2)TargetViewport_Extent >> MipLevel);
}

// Error Normalization.
// to have a similar error level between euclidean e.g. for color
// C0 = (0.58f, 0.46f, 0.28f), C1 = (0.28f, 0.96f, 0.43f)
// the color difference of euclidean with gamma corrected D_eu_g
// D_eu_g(C0, C1) = 0.38f
// the color difference of DeltaE_CIE2000 has
// DeltaE(C0,C1)  = 27.71
// To have the error on the same level, we multiply by 0.53f/ 37.85f = 1/70.78f, which is an experimental value
// of running 1e6 random pair of colors to get the mean of different measures where
// Mean_eu_g = 0.53f, Mean_DeltaE = 37.85f;
static float NormalizeDeltaEAndEuclideanDistanceToSameMean = 0.0141f;
#define NormalizeDeltaE2000(X) (NormalizeDeltaEAndEuclideanDistanceToSameMean*X)

// rgb is in linear space
float GetDeltaE(float3 LinearRGB0, float3 LinearRGB1, bool bNormalizeToEuclideanMean = true)
{

	float3 Lab0 = LinearRGB_2_LAB(LinearRGB0);
	float3 Lab1 = LinearRGB_2_LAB(LinearRGB1);

	float DeltaE = DeltaE_CIE2000(Lab0, Lab1);

	if (bNormalizeToEuclideanMean)
	{
		DeltaE = NormalizeDeltaE2000(DeltaE);
	}

	return DeltaE;
}

float GetDistance(float LumA, float LumB, int2 shift)
{
	float distance = 0;

#if DISTANCE_TYPE == DISTANCE_L1
	return abs(LumA - LumB);
#elif DISTANCE_TYPE == DISTANCE_L2
	return Square(LumA - LumB);
#elif DISTANCE_TYPE == DISTANCE_LOG_L1
	return log2(2 + abs(LumA - LumB));
#elif DISTANCE_TYPE == DISTANCE_LOG_REGULARIZED_L1
	return log2(2 + abs(LumA - LumB)) * (2 - exp(-0.02 * length2(shift)));
#else
#error Unsupported dinstance type between two pixels, please implement.
	return 0.0f;
#endif
}

float GetFinalMipDistance(float LumA, float LumB, int2 shift)
{
	return log2(2 + abs(LumA - LumB)) * (2 - exp(-0.02 * length2(shift)));
}

float GetDistance(float3 A, float3 B, int2 shift)
{

#if APPLY_SIMPLIFIED_GAMMA
	A = pow(abs(A), 1 / 2.2f);
	B = pow(abs(B), 1 / 2.2f);
#endif

#if DISTANCE_METRICS == METRICS_LUMINANCE
	float LumA = Luminance(A);
	float LumB = Luminance(B);
#elif DISTANCE_METRICS == METRICS_EUCLIDEAN
	float LumA = length(A - B);
	float LumB = 0;
#elif DISTANCE_METRICS == METRICS_PERCEPTION
	float LumA = GetDeltaE(A, B);
	float LumB = 0;
#endif
	return GetDistance(LumA, LumB, shift);
}

float GetFinalMipDistance(float3 A, float3 B, int2 shift)
{

#if APPLY_SIMPLIFIED_GAMMA
	A = pow(abs(A), 1 / 2.2f);
	B = pow(abs(B), 1 / 2.2f);
#endif

#if DISTANCE_METRICS == METRICS_LUMINANCE
	float LumA = Luminance(A);
	float LumB = Luminance(B);
#elif DISTANCE_METRICS == METRICS_EUCLIDEAN
	float LumA = length(A - B);
	float LumB = 0;
#elif DISTANCE_METRICS == METRICS_PERCEPTION
	float LumA = GetDeltaE(A, B);
	float LumB = 0;
#endif
	return GetFinalMipDistance(LumA, LumB, shift);
}

#if TEMPORAL_REPROJECTION_ALIGN || TEMPORAL_REPROJECTION_MERGE || TEMPORAL_REPROJECTION_RESOLVE || TEMPORAL_HIGHFREQ_REJECT

Texture2D<float4> PixelOffsetTexture;
Texture2D<float4> SourceTexture; 
Texture2D<float4> TargetTexture; 
SamplerState SharedTextureSampler;
uint MipLevel;

FPixelOffset GetPixelOffsetFromLowerMip(float2 Position, uint MipDiff)
{
	uint MipPosMultipler = 1u << (MipDiff);
	uint LowerMipLevel = MipLevel + MipDiff;

	float2 LowerMipPosition = 0.0f;

	BRANCH
	if (MipDiff == 0)
	{
		LowerMipPosition = Position;
	}
	else
	{
		// Keep the reprojection border of the coarse level close to zero
		LowerMipPosition = (Position - 1.0) / (float)MipPosMultipler - 0.5f;
	}

	FPixelOffset Offset = (FPixelOffset)0;

	float2 Extent = (float2)((uint2)TargetViewport_Extent >> LowerMipLevel);
	float2 UV = (LowerMipPosition + 0.5f) / Extent;
	float4 OffsetDist = Texture2DSampleBicubic(PixelOffsetTexture, SharedTextureSampler, UV, Extent, 1.0f / Extent);

	Offset.xy = MipPosMultipler * OffsetDist.xy;
	Offset.Dist = OffsetDist.z;
#if TOTAL_VARIATION || TEMPORAL_REPROJECTION_RESOLVE || TEMPORAL_HIGHFREQ_REJECT
	Offset.TV = OffsetDist.w;
#else
	Offset.TV = 1.0f;
#endif
	return Offset;
}

float3 GetTextureValue(Texture2D Tex, float2 Position)
{
	float2 BufferUV = GetBufferUVFromPosition(Position, MipLevel);
	return Texture2DSampleLevel(Tex, SharedTextureSampler, BufferUV, 0).rgb;
}

float3 GetTextureValueWithOffset(Texture2D Tex, float2 Position)
{
	// Get the postion offset from the previous miplevel texture
	FPixelOffset Offset = GetPixelOffsetFromLowerMip(Position, MIP_DIFF_DELTA);

	float2 BufferUV = GetBufferUVFromPosition(Position + Offset.xy, MipLevel);
	return Texture2DSampleLevel(Tex, SharedTextureSampler, BufferUV, 0).rgb;
}

#endif

#if TEMPORAL_PREPASS

Texture2D<float4> InputTexture;
Texture2D<float4> AlbedoTexture;
Texture2D<float4> NormalTexture;
RWStructuredBuffer<FPixelMaterialLightingFingerprint> RWVarianceMap;
int Iteration;

// Channel ranked luminance places the max value to g, and minimal to b
// In this way, the luminance is perception independent. The variance of (X,0,0) is the same to (0,X,0)
// instead of being smaller. 
float ChannelRankedLuminance(float3 Value)
{
	int Index = 0;
	Max3Index(Value, Index);
	Swap(Value[Index], Value.g); // swap max value to g
	Min3Index(Value, Index);
	Swap(Value[Index], Value.b); // swap min value to b

	return Luminance(Value);
}

// Note that the radiance, albedo and normal are the mean value.
float4 GetMeanFromTextures(int2 Position)
{
	float4 Value = 0;
#if VARIANCE_TYPE == RADIANCE_MULTI_CHANNEL
	Value  = InputTexture[Position];
#elif VARIANCE_TYPE == RADIANCE_ALBEDO_NORMAL_SINGLECHANNEL

#if RANKED_LUMINANCE_VARIANCE
	Value.x = ChannelRankedLuminance(InputTexture[Position].rgb);
	Value.y = ChannelRankedLuminance(AlbedoTexture[Position].rgb);
#else
	Value.x = Luminance(InputTexture[Position].rgb);
	Value.y = Luminance(AlbedoTexture[Position].rgb);
#endif
	//The length of the average normal indicates the variance
	//1: All pointing to the same direction
	//0: Uniform in all directions
	Value.z = length(NormalTexture[Position].rgb);
	
	// Special case handling:
	// 1. Regions without normal has a zero length, like sky
	// 2. The first frame
	if (Value.z == 0.0f || Iteration == 0)
	{
		Value.z = 1.0f;
	}
	
	// Alpha channel needs denoising as well.
	Value.w = InputTexture[Position].a;
#else
	#error "Not implemented" 
#endif
	return Value;
}

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void TemporalPrepassCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = (int2)DT_ID.xy;
	int2 MaxPosition = int2(TargetViewport_Extent);
	if (all(DT_ID.xy < MaxPosition))
	{
		int index = Position.x + Position.y * TargetViewport_Extent.x;
		
		float4 Mean = GetMeanFromTextures(Position);

#if PREPASS_PHASE == PREPASS_PHASE_INIT

		FPixelMaterialLightingFingerprint VarianceInfo = (FPixelMaterialLightingFingerprint)0;
		VarianceInfo.Mean = Mean;
		RWVarianceMap[index] = VarianceInfo;

#elif PREPASS_PHASE == PREPASS_PHASE_UPDATE

		FPixelMaterialLightingFingerprint VarianceInfo = RWVarianceMap[index];
		
		float4 PreviousMean = VarianceInfo.Mean;
		float4 CurrentValue = (Mean - PreviousMean) * Iteration + Mean;
		
		VarianceInfo.Mean = Mean;
		VarianceInfo.Var += (CurrentValue - VarianceInfo.Mean)*(CurrentValue - PreviousMean);

		RWVarianceMap[index] = VarianceInfo;
#else
	#error "Not implemented" 
#endif
	}
}
#endif // TEMPORAL_PREPASS

#if PREPASS_GENERATE_TEXTURE
RWTexture2D<float4> OutputTexture;
StructuredBuffer<FPixelMaterialLightingFingerprint> VarianceMap;
int Iteration;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void PrepassGenerateTextureCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = (int2)DT_ID.xy;
	int2 MaxPosition = int2(TargetViewport_Extent);
	if (all(DT_ID.xy < MaxPosition))
	{
		int index = Position.x + Position.y * TargetViewport_Extent.x;
		FPixelMaterialLightingFingerprint VarianceInfo = VarianceMap[index];

		// VarianceInfo.Var = sigma^2 * n.
		// Std of mean = sqrt(sigma^2/n) = sqrt(VarianceInfo.Var/n^2)
		float4 StandDerivation= sqrt(VarianceInfo.Var) / (Iteration + 1);

		OutputTexture[Position] = StandDerivation;
	}
}
#endif // PREPASS_GENERATE_TEXTURE

#if TEMPORAL_FEATURE_FUSION
//Use Albedo, Normal, and Radiance Texture to create another feature vector
// such that the feature vector leads to the best motion vector estimation.

Texture2D<float4> AlbedoTexture_0;
Texture2D<float4> AlbedoTexture_1;

Texture2D<float4> NormalTexture_0;
Texture2D<float4> NormalTexture_1;

Texture2D<float4> RadianceTexture_0;
Texture2D<float4> RadianceTexture_1;

StructuredBuffer<FPixelMaterialLightingFingerprint> VarianceMap_0;
StructuredBuffer<FPixelMaterialLightingFingerprint> VarianceMap_1;


Texture2D<float4> LastDenoisedRadiance;

SamplerState SharedTextureSampler;

RWTexture2D<float4> OutputTexture_0;
RWTexture2D<float4> OutputTexture_1;

#define SOURCE(x) x##_0[Position]
#define TARGET(x) x##_1[Position]

#define SOURCE_BUFFER(B) B##_0[Position.x+int(Position.y*TargetViewport_Extent.x)]
#define TARGET_BUFFER(B) B##_1[Position.x+int(Position.y*TargetViewport_Extent.x)]

float4 Fuse(float4 Albedo, float4 Normal, float4 Radiance, float4 History, FPixelMaterialLightingFingerprint Fingerprint)
{
	float4 combined = 0;
	//TODO: Derive good feature fusion algorithm.
	combined.rgb = sqrt(Fingerprint.Var.rgb / (Fingerprint.Mean.w + 1));

	return combined;
}

float4 FuseSource(int2 Position)
{
	float4 Albedo = SOURCE(AlbedoTexture);
	float4 Normal = SOURCE(NormalTexture);
	float4 Radiance = SOURCE(RadianceTexture);
	float4 History = LastDenoisedRadiance[Position];
	FPixelMaterialLightingFingerprint Fingerprint = SOURCE_BUFFER(VarianceMap);

	return Fuse(Albedo, Normal, Radiance, History,Fingerprint);
}

float4 FuseTarget(int2 Position)
{
	float4 Albedo = TARGET(AlbedoTexture);
	float4 Normal = TARGET(NormalTexture);
	float4 Radiance = TARGET(RadianceTexture);
	float4 History = -1.0f;
	FPixelMaterialLightingFingerprint Fingerprint = TARGET_BUFFER(VarianceMap);

	return Fuse(Albedo, Normal, Radiance, History, Fingerprint);
}

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void TemporalFeatureFusionCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = (int2)TargetViewport_Extent;
	if (all(Position < MaxPosition))
	{
		SOURCE(OutputTexture) = FuseSource(Position);
		TARGET(OutputTexture) = FuseTarget(Position);
	}
}

#endif // TEMPORAL_FEATURE_FUSION

#if TEMPORAL_REPROJECTION_ALIGN

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void ReprojectionAlignCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int SliceId = DT_ID.z;
	uint3 MaxPosition = uint3(((uint2)TargetViewport_Extent.xy) >> MipLevel, K_NUM_OF_TEXTURES_PER_PASS);

	if (all(DT_ID.xyz < MaxPosition))
	{
		float3 Target = GetTextureValue(TargetTexture, Position)*View.PreExposure;
		float4 Distances = 0.0f;

		UNROLL
			for (int i = 0; i < K_NUM_OF_SHIFTS_PER_SLICE; ++i)
			{
				int2 Shift = Shifts[SliceId * K_NUM_OF_SHIFTS_PER_SLICE + i];
				int2 ShiftPosition = GetShiftedPosition(Position, Shift);
				float3 Source = GetTextureValueWithOffset(SourceTexture, ShiftPosition)*View.PreExposure;

				if (MipLevel != 0)
				{
					Distances[i] = GetDistance(Target, Source, Shift);
				}
				else
				{
					Distances[i] = GetFinalMipDistance(Target, Source, Shift);
				}
			}

		SaveOffsets(Position, SliceId * K_NUM_OF_SHIFTS_PER_SLICE, Distances);
	}
}
#endif // TEMPORAL_REPROJECTION_ALIGN

#if TEMPORAL_REPROJECTION_BLUR

Texture2D<float4> InputTexture; // Previous accumulation
RWTexture2D<float4> OutputTexture; // new observation
SamplerState SharedTextureSampler;
uint MipLevel;

float4 Get3x3Blur(int2 Position)
{
	float2 BufferUV = GetBufferUVFromPosition(Position, MipLevel);
	float2 HalfPixel = 0.5f * (1u << MipLevel) * TargetViewport_ExtentInverse;

#if 0
	//Sampling pattern and weights
	// 1   1
	//   4
	// 1   1
	float4 Sum = Texture2DSampleLevel(InputTexture, SharedTextureSampler, BufferUV, 0) * 4;
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + float2(-HalfPixel.x, -HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + float2(HalfPixel.x, -HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + float2(-HalfPixel.x, HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + float2(HalfPixel.x, HalfPixel.y)), 0);

	return Sum / 8.0f;

#elif DIRECT_COPY

	return InputTexture[Position];

#else

	float4 Sum = 0;
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(-HalfPixel.x, -HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(-HalfPixel.x, 0)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(-HalfPixel.x, HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(0, -HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(0, 0)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(0, HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(HalfPixel.x, -HalfPixel.y)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(HalfPixel.x, 0)), 0);
	Sum += Texture2DSampleLevel(InputTexture, SharedTextureSampler,
		(BufferUV + 2.0 * float2(HalfPixel.x, HalfPixel.y)), 0);

	return Sum / 9.0f;
#endif
	
}

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void ReprojectionBlurCS(uint3 DT_ID : SV_DispatchThreadID)
{

	int2 Position = (int2)DT_ID.xy;
	int2 MaxPosition = int2(((uint2)TargetViewport_Extent) >> MipLevel);
	if (all(Position.xy < MaxPosition))
	{
		float4 Blur = Get3x3Blur(Position);
		OutputTexture[Position] = Blur;
	}
}
#endif // TEMPORAL_REPROJECTION_ALIGN

#if TEMPORAL_REPROJECTION_MERGE

RWTexture2D<float4> RWPixelOffsetTexture;
uint PatchId;
uint PatchCount;

// For a three level mip chain and a delta of MIP_DIFF_DELTA.
// fetch the patch. E.g., 
// 0-0-0 fetch the patch with the minimal distance
// 0-1-0 fetch the second best for the second mip level

uint GetTheKthPatchBasedOnMipAndPatchId()
{
	uint Mip2PatchIndex = PatchId % 4;
	uint Mip0PatchIndex = PatchId / 4;
	const uint TopMipLevel = 0;
	uint TheKthPatchForMipLevel = lerp(
		lerp(0,																// All others
			Mip2PatchIndex,	MipLevel == (TopMipLevel + MIP_DIFF_DELTA)),	// 2nd mip level
			Mip0PatchIndex,	MipLevel == TopMipLevel					  );	// Most fine grain level

	return TheKthPatchForMipLevel;
}

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void ReprojectionMergeCS(uint3 DT_ID : SV_DispatchThreadID)
{
	// Keep the debug here as the reproject still needs some work around borders for better  history reuse.
	bool DebugMotionVector = false;

	int2 Position = DT_ID.xy;
	int2 MaxPosition = int2(((uint2)TargetViewport_Extent) >> MipLevel);
	if (all(Position < MaxPosition))
	{
		uint Kth = GetTheKthPatchBasedOnMipAndPatchId();

		FQuadricFittingContext Context = GetQuadricFittingContext(Position, Kth);

		// Add up the old offset with the new one.
		FPixelOffset OffsetFromLowerMip = GetPixelOffsetFromLowerMip(Position, MIP_DIFF_DELTA);

		// the offset is from target to source
		FPixelOffset Offset = GetQuadricFittingOffset(Context);

		float4 PixelDiff = float4(Offset.xy + OffsetFromLowerMip.xy, Offset.Dist, length(Offset.xy + OffsetFromLowerMip.xy));

#if TOTAL_VARIATION
		float TotalVariation = GetTotalVariation(Position + PixelDiff.xy, Context);
		PixelDiff.w = TotalVariation;
#else
		PixelDiff.w = 1.0f;
#endif
		if (!DebugMotionVector)
		{
			RWPixelOffsetTexture[Position] = PixelDiff;
		}
		else
		{
			if (MipLevel >= 6)
			{
				RWPixelOffsetTexture[Position] = PixelDiff;
			}
			else
			{
				RWPixelOffsetTexture[Position] = float4(OffsetFromLowerMip.xy, OffsetFromLowerMip.Dist, length(OffsetFromLowerMip.xy));
			}
		}
	}
}
#endif // TEMPORAL_REPROJECTION_MERGE

#if TEMPORAL_HIGHFREQ_REJECT

RWTexture2D<float4> OutputTexture;
float HighFrequencyCutoffDeltaE;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void TemporalHighFrequencyRejectCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = (int2)TargetViewport_Extent;
	if (all(Position < MaxPosition))
	{

		FPixelOffset Offset = GetPixelOffsetFromLowerMip(Position, 0);

		float2 Pos = Position + Offset.xy;

		float2 UV = (Pos + 0.5f) * TargetViewport_ExtentInverse;
		float4 History_Warped = Texture2DSampleBicubic(SourceTexture, SharedTextureSampler, UV, TargetViewport_Extent, TargetViewport_ExtentInverse);

		float4 Target = TargetTexture[Position];

		//calculate the distance between warped source and target
		const bool bNormalizeDeltaE = true;
		float dENormalized = GetDeltaE(History_Warped.rgb*View.PreExposure, Target.rgb*View.PreExposure, bNormalizeDeltaE);

		// since the warping of the source image is based on the distance of a region with weights
		// it is a matching based on low frequency information. High frequency detail is not well
		// utilized. So in this CS, we try to use the high frequency information to correct the blending
		// factor per pixel. E.g., when albedo is used, we expect a high matching.
		// TODO: Explore different buffers.
		OutputTexture[Position] = float4(dENormalized,
			dENormalized >= NormalizeDeltaE2000(HighFrequencyCutoffDeltaE),
			Luminance(History_Warped.rgb),
			Luminance(Target.rgb));
	}

}
#endif // TEMPORAL_HIGHFREQ_REJECT


#if TEMPORAL_REPROJECTION_RESOLVE
Texture2D<float4> HighFrequencyRejectMap;
RWTexture2D<float4> OutputTexture;
float Alpha;
float Kappa;
float Eta;
float HighFrequencyCutoffDeltaE;
uint PatchId;
uint PatchCount;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void TemporalResolveCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = (int2)TargetViewport_Extent;
	if (all(Position < MaxPosition))
	{

		FPixelOffset Offset = GetPixelOffsetFromLowerMip(Position, 0);

		float2 Pos = Position +Offset.xy;		
		float2 UV = (Pos + 0.5f) * TargetViewport_ExtentInverse;
		float4 Sampling = Texture2DSampleBicubic(SourceTexture, SharedTextureSampler, UV, TargetViewport_Extent, TargetViewport_ExtentInverse);
		
		float m = max(Kappa * (Offset.Dist - Eta), 0.0f);
		m = 2 * m / (1 + m);
		float FinalAlpha = clamp(Alpha * (1 - m) * Offset.TV, 0, 1);
		
		float dEHighFrequency = HighFrequencyRejectMap[Position].x;

		// Determine the final alpha based on perception error
		// 1. if the high frequency difference is larger than 10dE reject
		// 2. if the difference is smaller than the high frequency cutoff, use the weight based on low frequency
		// 3. otherwise, we move the target color to the source by a distance at most 0.5 dE
		// TODO: lerp in Lab space instead of linear RGB space?
		FinalAlpha = lerp(FinalAlpha,
			min(FinalAlpha, NormalizeDeltaE2000(0.5f) / dEHighFrequency),
			dEHighFrequency >= NormalizeDeltaE2000(HighFrequencyCutoffDeltaE));

		FinalAlpha = lerp(FinalAlpha, 0, dEHighFrequency >= NormalizeDeltaE2000(10.0f));

		float4 Accumulation = 0.0f;
		float Weight = 0.0f;
		float4 TargetValue = TargetTexture[Position];

		BRANCH
		if  (PatchCount == 1)
		{	
			// Use exponential moving to low pass the signal to create more
			// temporal stable results.

			Accumulation = (1 - FinalAlpha) * TargetValue + FinalAlpha * Sampling;
			Weight = FinalAlpha;
		}
		else
		{
			// When we have multiple patches, use bilaterial filter to combine them
			// TODO: improve high frequency information for this mode.
			
			Accumulation = TargetValue;
			Weight = (PatchId == 0) ? 1 : Accumulation.w;
		
			Accumulation += FinalAlpha * Sampling;
			Weight += FinalAlpha;

			if (PatchId + 1 == PatchCount)
			{
				Accumulation /= Weight;
			}
		}

		OutputTexture[Position] = float4(max(Accumulation.xyz, 0.0f), Weight);
	}
}
#endif // TEMPORAL_REPROJECTION_RESOLVE

// TODO: Ablation test
float GetWeightMultiplier(float SourceTV, float TargetSourceTV)
{
	// Use less history if total variation is larger than reference
	float delta = -(TargetSourceTV - SourceTV);
	float depth = 0.2f;
	return 1 / (1 + exp(-10.0 * delta)) * depth + (1 - depth / 2.0);
}

#if MOTION_VECTOR_SUBTRACT
RWTexture2D<float4> Minuend;
Texture2D<float4> Subtrahend;
uint MipLevel;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void MotionVectorSubtractCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = int2(((uint2)TargetViewport_Extent) >> MipLevel);
	if (all(Position < MaxPosition))
	{
		float4 M = Minuend[Position];
		float4 S = Subtrahend[Position];

		float4 M_S = float4(M.x - S.x, M.y - S.y, M.z, GetWeightMultiplier(S.w, M.w));

		// Do not estimate the TV at the border when history is not available
		const uint NumOfIgnoredBorderPixels = 12;
		if (any(Position > TargetViewport_Extent - NumOfIgnoredBorderPixels) ||
			any(Position - NumOfIgnoredBorderPixels < float2(0, 0)))
		{
			M_S.w = 1.0f;
		}

		Minuend[Position] = M_S;
	}
}
#endif // MOTION_VECTOR_SUBTRACT

#if VISUALIZE_MOTIONVECTOR

// h = [0, 1], s = [0 ,1], v = [0, 1]
// h: hue, 
// s: saturate
// v: value
float3 hsv2rgb(float h, float s, float v)
{
	h = 360.0 * h;

	float C = v * s;
	float X = C * (1 - abs( (h/60) % 2 - 1));
	float m = v - C;

	float3 RGB = 0;
	if (h >= 0 && h < 60)
	{
		RGB.r = C;
		RGB.g = X;
	}
	else if (h >= 60 && h < 120)
	{
		RGB.r = X;
		RGB.g = C;
	}
	else if (h >= 120 && h < 180)
	{
		RGB.g = C;
		RGB.b = X;
	}
	else if (h >= 180 && h < 240)
	{
		RGB.g = X;
		RGB.b = C;
	}
	else if (h >= 240 && h < 300)
	{
		RGB.r = X;
		RGB.b = C;
	}
	else if (h >= 300 && h < 360)
	{
		RGB.r = C;
		RGB.b = X;
	}

	RGB += m;
	return RGB;
}

//:adapted from https://www.shadertoy.com/view/MsS3Wc by iq
float3 hsv2rgb_smooth(float3 c)
{
	float3 rgb = clamp(abs( (c.x * 6.0 + float3(0.0, 4.0, 2.0)) % 6.0 - 3.0) - 1.0, 0.0, 1.0);

	rgb = rgb * rgb * (3.0 - 2.0 * rgb); // cubic smoothing	

	return c.z * lerp(float3(1.0, 1.0, 1.0), rgb, c.y);
}


Texture2D<float4> TemporalDenoisingMotionVector;
Texture2D<float4> DenoisedTexture;

#define VISUALIZE_VECTOR 1
#define VISUALIZE_COLOR_ENCODING 1
#define VECTOR_GRID_LENGTH 32
#define COLOR_METER_LENGTH 50

void VisualizePathTracingMotionVector(
	in noperspective float2 UV : TEXCOORD0,
	out float4 OutColor : SV_Target0
)
{
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	int3 TexCoord = int3(UV * BufferSize - View.ViewRectMin.xy, 0);

	float4 Velocity = TemporalDenoisingMotionVector.Load(TexCoord);

	// draw the denoised scene
	OutColor.rgb = DenoisedTexture.Load(TexCoord).rgb * View.PreExposure;

#if VISUALIZE_COLOR_ENCODING
	float s, h, v;
	if (length(Velocity.xy) >= 0.001) // ignore small velocity color encoding
	{
#if 0
		s = clamp(lengthFast(float3(Velocity.xy, 0.0f)), 0.0f, 50.0f) / 50.0f;
		h = atan2(Velocity.y, Velocity.x) / 3.1415926f * 0.5f + 0.5f;
		v = 1.0f;
		OutColor = float4(hsv2rgb(h, s, v), 1.0f);// float4(s, h, Velocity.b, 1.0f);
		OutColor.rgb = pow(1 - OutColor.rgb, 1 / 2.2f);
#else
		v = pow(clamp(length(float3(Velocity.xy, 0.0f)), 0.0f, 50.0f) / 50.0f, 1 / 2.2f);
		h = atan2(Velocity.y, -Velocity.x) / 3.1415927f;
		h = h < 0 ? ((h + 2) / 2) : h / 2.0;
		s = 1.0f;
		OutColor = float4(hsv2rgb_smooth(float3(h, s, v)), 1.0f);// float4(s, h, Velocity.b, 1.0f);
		OutColor.rgb = pow(OutColor.rgb, Velocity.z == 0? 1.0f: Velocity.z);
#endif
	}
	else
	{
		OutColor.rgb = 0.0f;
	}

	// add motion color meter
	if (length(TexCoord.xy - COLOR_METER_LENGTH) < COLOR_METER_LENGTH)
	{
		float2 Vector = TexCoord.xy - COLOR_METER_LENGTH;
#if 0
		s = length(float3(Vector, 0)) / COLOR_METER_LENGTH;
		h = atan2(-Vector.y, -Vector.x) / 3.1415927f * 0.5f + 0.4999f;
		v = 1.0f;
		OutColor = float4(hsv2rgb(h, s, v), 1.0f);// float4(s, h, Velocity.b, 1.0f);
		OutColor.rgb = pow(1 - OutColor.rgb, 1 / 2.2f);
#else
		v = pow(length(float3(Vector, 0)) / COLOR_METER_LENGTH, 1 / 2.2f);
		h = atan2(-Vector.y, Vector.x) / 3.1415927f;
		h = h < 0 ? ((h + 2) / 2) : h / 2.0;
		s = 1.0f;
		OutColor = float4(hsv2rgb_smooth(float3(h, s, v)), 1.0f);// float4(s, h, Velocity.b, 1.0f);
		OutColor.rgb = pow(OutColor.rgb, 1.0f);
#endif
	}
#endif 

#if VISUALIZE_VECTOR
	
	int2 P0 = (TexCoord.xy / VECTOR_GRID_LENGTH) * VECTOR_GRID_LENGTH + VECTOR_GRID_LENGTH / 2;
	Velocity = TemporalDenoisingMotionVector.Load(int3(P0, 0));
	float2 P1 = (float2)P0 + Velocity.xy;
	float2 N = normalize(float2(Velocity.y, -Velocity.x));
	float2 P2 = (float2)TexCoord.xy;
	
#if 0	// plot actual velocity
	float Dist = abs(dot(N, P2 - P0));
	float Lambda = dot(P2 - P0, P1 - P0) / dot(Velocity.xy, Velocity.xy);
	float P02P1Length = length(Velocity.xy);

	if (P02P1Length > 0.001)
	{
		if (Dist <= 3 && Lambda >= -0.1 && Lambda <= 1.1)
		{
			
			if (Dist <= 1.0)
			{
				OutColor = lerp(float4(0.5f, 0.5f, 0.5f, 1.0f), float4(1.0f, 0.0f, 0.0f, 1.0f), Lambda);
			}

			if (length(P2-P0) <= 2)
			{
				OutColor = 0.6f;
			}
			
			if (length(P2-P1)<=2)
			{
				OutColor = float4(1.0, 0.0, 0.0, 1.0f);
			}
			

		}
	}
#else
	float3 LineColor = float3(0.7, 0.7, 0.7);

	// points into the movement direction, movement direction is the inverse of the reprojection velocity.
	float2 PixelDirection = -Velocity.xy;
	float2 PixelPosAtTileCenter = P0;
	float2 PixelPos = P2;
	// arrow
	{
		float2 PerpPixelDirection = float2(PixelDirection.y, -PixelDirection.x);
		float2 DirectionInTile = PixelPos - PixelPosAtTileCenter;

		float DistOnLine = dot(normalize(-PixelDirection), DirectionInTile) + length(PixelDirection);

		bool bArrowHead = DistOnLine < 8;

		float LocalThickness = 1 + (frac(DistOnLine / 8) * 8) * 0.25f;

		float PerpDirectionMask = saturate(LocalThickness - abs(dot(normalize(PerpPixelDirection), DirectionInTile)));
		float DirectionMask = saturate(length(PixelDirection) - length(DirectionInTile));

		float3 LineMask = PerpDirectionMask * DirectionMask;
		OutColor.rgb = lerp(OutColor.rgb, LineColor, LineMask);
	}

	// previous pos is a dot
	{
		float3 DotColor = float3(0, 1, 0);
		// PixelPos of the previous position
		float2 PreviousPixelPos = PixelPosAtTileCenter - PixelDirection;
		float Dist = length(PreviousPixelPos - PixelPos);
		OutColor.rgb = lerp(OutColor.rgb, LineColor, saturate(3 - Dist));
		OutColor.rgb = lerp(OutColor.rgb, 0, saturate(1.5f - Dist));
	}
#endif

#endif

}

#endif

#if VISUALIZE_WARPING

Texture2D<float4> TemporalDenoisingMotionVector;
Texture2D<float4> DenoisedTexture;
Texture2D<float4> SourceTexture;
SamplerState SharedTextureSampler;

void FVisualizeWarpingPS(
	in noperspective float2 UV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	int3 TexCoord = int3(UV * BufferSize - View.ViewRectMin.xy, 0);

	float4 Velocity = TemporalDenoisingMotionVector.Load(TexCoord);

	float2 Pos = TexCoord.xy + Velocity.xy;

	float2 SourceUV = (Pos + 0.5f + View.ViewRectMin.xy) * TargetViewport_ExtentInverse;
	float4 Sampling = Texture2DSampleBicubic(SourceTexture, SharedTextureSampler, SourceUV, TargetViewport_Extent, TargetViewport_ExtentInverse);

	OutColor.rgb = Sampling.rgb * View.PreExposure;
	OutColor.a = 1.0f;
}
#endif

#if SPATIAL_DENOISING

Texture2D<float4> InputTexture;
Texture2D<float4> InputNormal;
Texture2D<float4> InputAlbedo;
RWTexture2D<float4> OutputTexture;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void SpatialDenoiserCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = (int2)TargetViewport_Extent;
	if (all(Position < MaxPosition))
	{
		OutputTexture[Position] = InputTexture[Position];
	}
}
#endif // SPATIAL_DENOISING

#if PREPROCESS_BUFFER

RWTexture2D<float4> NormalTexture;
float Width;
float Height;

[numthreads(THREAD_SIZE_X, THREAD_SIZE_Y, 1)]
void ConvertWorldSpaceNormalToCameraSpaceCS(uint3 DT_ID : SV_DispatchThreadID)
{
	int2 Position = DT_ID.xy;
	int2 MaxPosition = int2(Width, Height);
	if (all(Position < MaxPosition))
	{
		float4 Normal = float4(NormalTexture[Position].xyz, 0.0f);
		NormalTexture[Position] = float4(mul(Normal, View.TranslatedWorldToCameraView).xyz, 0.0f);
	}
}
#endif
