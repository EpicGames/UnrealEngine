// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

uint2 DispatchDim;
uint2 TileSize;
uint ScanlineStride;
Texture2D<float4> InputTexture;
RWTexture2D<float4> OutputTexture;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void PathTracingSwizzleScanlinesCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId >= DispatchDim))
	{
		return;
	}

	// shuffle the scanlines tile by tile
	int TileBaseOffset = (DispatchThreadId.y / TileSize.y) * TileSize.y;
	int TileOffset = DispatchThreadId.y - TileBaseOffset;

	int TileDispatchSize = min(TileSize.y, DispatchDim.y - TileBaseOffset);

	int TileScanlineCount = (TileDispatchSize + ScanlineStride - 1) / ScanlineStride;


	// Input texture has ScanlineStride copies of the image (as rendered by each GPU) arranged vertically
	// Swizzle this back into one image by interleaving the scanlines
	int g = TileOffset / ScanlineStride;
	int y = TileOffset - g * ScanlineStride; // TileOffset% ScanlineStride;
	int ScanlineY = y * TileScanlineCount + g + TileBaseOffset;

	OutputTexture[DispatchThreadId] = InputTexture.Load(int3(DispatchThreadId.x, ScanlineY, 0));
}
