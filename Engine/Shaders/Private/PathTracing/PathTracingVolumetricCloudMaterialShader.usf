// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1
#define PATH_TRACER_USE_CLOUD_SHADER 1

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingShaderUtils.ush"
// Provide aliases for the parameters that now come through a shader buffer 

#define FogDensity 		PathTracingCloudParameters.FogDensity
#define FogHeight 		PathTracingCloudParameters.FogHeight
#define FogFalloff		PathTracingCloudParameters.FogFalloff
#define FogAlbedo		PathTracingCloudParameters.FogAlbedo
#define FogPhaseG		PathTracingCloudParameters.FogPhaseG
#define FogCenter		PathTracingCloudParameters.FogCenter
#define FogMinZ			PathTracingCloudParameters.FogMinZ
#define FogMaxZ			PathTracingCloudParameters.FogMaxZ
#define FogRadius 		PathTracingCloudParameters.FogRadius
#define FogFalloffClamp	PathTracingCloudParameters.FogFalloffClamp

#define PlanetCenterTranslatedWorldHi 	PathTracingCloudParameters.PlanetCenterTranslatedWorldHi
#define PlanetCenterTranslatedWorldLo 	PathTracingCloudParameters.PlanetCenterTranslatedWorldLo
#define Atmosphere 						PathTracingCloudParameters

#define CloudAccelerationMap 			PathTracingCloudParameters.CloudAccelerationMap
#define CloudAccelerationMapSampler 	PathTracingCloudParameters.CloudAccelerationMapSampler

#define CloudClipX						PathTracingCloudParameters.CloudClipX						
#define CloudClipY						PathTracingCloudParameters.CloudClipY
#define CloudClipZ						PathTracingCloudParameters.CloudClipZ

#define CloudLayerBotKm					PathTracingCloudParameters.CloudLayerBotKm
#define CloudLayerTopKm					PathTracingCloudParameters.CloudLayerTopKm
#define CloudClipDistKm                 PathTracingCloudParameters.CloudClipDistKm
#define CloudClipRadiusKm               PathTracingCloudParameters.CloudClipRadiusKm

#define CloudClipCenterKm				PathTracingCloudParameters.CloudClipCenterKm

#define CloudTracingMaxDistance			PathTracingCloudParameters.CloudTracingMaxDistance
#define CloudRoughnessCutoff			PathTracingCloudParameters.CloudRoughnessCutoff

#define CloudAccelMapResolution			PathTracingCloudParameters.CloudAccelMapResolution
#define CloudVoxelWidth					PathTracingCloudParameters.CloudVoxelWidth
#define CloudInvVoxelWidth				PathTracingCloudParameters.CloudInvVoxelWidth


#include "./Volume/PathTracingCloudsMaterialCommon.ush"
#include "./Volume/PathTracingFog.ush"
#include "./Volume/PathTracingAtmosphere.ush"
#include "./Volume/PathTracingVolumeSampling.ush"

#include "./PathTracingCommon.ush"

FVolumeShadedResult CloudsGetDensity(float3 TranslatedWorldPos)
{

#if CLOUD_VISUALIZATION_SHADER == 1
	float3 P = mul(GetCloudClipBasis(), TranslatedWorldPos); // WorldToTangent
	float2 UV = (P.xy * CM_TO_SKY_UNIT + CloudClipDistKm.xx) / (2 * CloudClipDistKm);
	const int Level = 0;
	float4 Data = CloudAccelerationMap.SampleLevel(CloudAccelerationMapSampler, UV, Level);
	
	float Z = P.z * CM_TO_SKY_UNIT;
	float Density = (Z > Data.z && Z < Data.w) ? Data.x : 0.0;
	FVolumeShadedResult Result = (FVolumeShadedResult)0;

	Result.SigmaT = Density;
	Result.SigmaSDualHG = 0.85 * Density;
	return Result;
#else
	FDFVector3 AbsoluteWorldPosition = DFFastSubtract(TranslatedWorldPos, ResolvedView.PreViewTranslation);

	// Measure HeightKm as the distance to the planet center
	const FDFVector3 OC = DFMultiply(DFSubtract(TranslatedWorldPos, MakeDFVector3(PlanetCenterTranslatedWorldHi, PlanetCenterTranslatedWorldLo)), CM_TO_SKY_UNIT);
	const float H = DFLengthDemote(OC); // HeightKm
	float B = CloudLayerBotKm;
	float T = CloudLayerTopKm;

	FSampleCloudMaterialResult CloudResult = SampleCloudMaterial(AbsoluteWorldPosition, H, B, T);
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT        = CloudResult.Extinction;
	Result.SigmaSDualHG  = CloudResult.Albedo * Result.SigmaT;
	Result.Emission      = CloudResult.Emission;
#if MATERIAL_VOLUMETRIC_ADVANCED
	Result.DualPhaseData = float3(CloudResult.PhaseG1, CloudResult.PhaseG2, CloudResult.PhaseBlend);
#endif
	return Result;
#endif
}

RAY_TRACING_ENTRY_CALLABLE(PathTracingVolumetricCloudMaterialShader, FPackedPathTracingPayload, PackedPayload)
{
	ResolvedView = ResolveView();

	// Ray marching happens inside here so that we only have to pay for the cost of callable shading _once_
	// This comes at the cost of some duplication of the logic
	// TODO: Can we use HLSL templates to merge some of the duplicated logic?

	FRandomSequence RandSeq = PackedPayload.GetVolumetricCallableShaderInputRandSequence();
	FRayDesc Ray = PackedPayload.GetVolumetricCallableShaderInputRay();
	float3 PathThroughput = PackedPayload.GetVolumetricCallableShaderInputThroughput();
	float CloudDensity = PackedPayload.GetVolumetricCallableShaderInputCloudDensity();
	const uint Flags = PackedPayload.GetVolumetricCallableShaderInputFlags();

	const int Bounce = (Flags & PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_MASK) >> PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_SHIFT;
	const bool bIsCameraRay = Bounce == 0;

	// TODO: evaluate if it would be better to make a second shader for this (so that we don't get two copies of the shader eval in inner loops)
	// TODO: evaluate if voxel traversal is worth it for transmittance
	if (Flags & PATH_TRACER_VOLUME_CALLABLE_FLAGS_TRANSMITTANCE)
	{
		const float CloudFactor = PackedPayload.GetVolumetricCallableShaderInputCloudFactor();
		// Reset payload
		PackedPayload = (FPackedPathTracingPayload)0;

		// Transmittance loop for the cloud shader only
		// NOTE: We have specialized this since we know our upper bound is a scalar value, most of the pdf math cancels out and we can take a faster path
		float TMin = Ray.TMin;
		float TMax = Ray.TMax;
		float SigmaBar = CloudDensity * CloudFactor, InvSigmaBar = rcp(SigmaBar);
		float3 Throughput = PathThroughput;
		for (int Step = 0; Step < PathTracingCloudParameters.MaxRaymarchSteps; Step++)
		{
			/* take stochastic steps along the ray to estimate transmittance (null scattering) */
			/* Sample the distance to the next interaction */										
			float RandValue = RandSeq.Get1D();
			TMin -= log(1 - RandValue) * InvSigmaBar;
			if (TMin >= TMax)
			{
				/* exit the ray marching loop */
				break;
			}
			/* our ray marching step stayed inside the atmo and is still in front of the next hit */
			float3 WorldPos = Ray.Origin + TMin * Ray.Direction;
			/* clamp to make sure we never exceed the majorant (should not be the case, but need to avoid any possible numerical issues) */
			float3 SigmaT = min(CloudFactor * CloudsGetDensity(WorldPos).SigmaT, SigmaBar);
			/* keep tracing through the volume */
			Throughput *= 1.0 - SigmaT * InvSigmaBar;
			/* Encourage early termination of paths with low contribution */
			Throughput *= LowThroughputClampFactor(Throughput);
			if (!any(Throughput > 0))
			{
				break;
			}
		}

		// Do we need to know the sample at the _end_ of the ray interval? capture it first in case it does not scatter any light
		if (Flags & PATH_TRACER_VOLUME_CALLABLE_FLAGS_GET_SAMPLE)
		{
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
			FCloudMaterialMSParams MS = (FCloudMaterialMSParams)0; 
			if (PathTracingCloudParameters.CloudShaderMultipleScatterApproxEnabled)
			{
				MS = CloudMaterialGetMSParams();
			}
			FRISContext HitSample = InitRISContext(RandSeq.Get1D());
			float3 FinalThroughput = 0;
#endif

			float3 TranslatedWorldPos = Ray.Origin + Ray.Direction * Ray.TMax;
			FVolumeShadedResult CloudResult = (FVolumeShadedResult)0; 
			MergeVolumeShadedResult(CloudResult, CloudsGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_CLOUDS) != 0);
			FVolumeShadedResult Result = CloudResult;
			if (Flags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
			{
				MergeVolumeShadedResult(Result, AtmosphereGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE) != 0);
			}
			if (Flags & PATH_TRACER_VOLUME_ENABLE_FOG)
			{
				MergeVolumeShadedResult(Result, FogGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_FOG) != 0);
			}

			float3 SigmaS = min(Result.SigmaSRayleigh + Result.SigmaSHG + Result.SigmaSDualHG, Result.SigmaT);
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
			float3 Contrib = Throughput * SigmaS;
			float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
			if (HitSample.Accept(SelectionWeight))
#else
			Throughput *= SigmaS;
#endif
			{
				const float3 RayleighWeight = select(SigmaS > 0.0, Result.SigmaSRayleigh / SigmaS, 0.0);
				const float3 HGWeight       = select(SigmaS > 0.0, Result.SigmaSHG       / SigmaS, 0.0);
				const float3 DualHGWeight   = select(SigmaS > 0.0, Result.SigmaSDualHG   / SigmaS, 0.0);
				PackedPayload.SetVolumetricCallableShaderOutputSample(Ray.TMax, RayleighWeight, HGWeight, Result.PhaseG, DualHGWeight, Result.DualPhaseData, 0.0);
				PackedPayload.SetVolumetricCallableShaderOutputCloudDensityFactor(1.0);
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
				FinalThroughput = Contrib / SelectionWeight;
#endif
			}
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
			float S = MS.ScattFactor, E = MS.ExtinFactor, P = MS.PhaseFactor;
			for (int ms = 1; ms <= MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT; ms++)
			{
				CloudResult.SigmaSDualHG *= S;

				// Propose a candidate hit with just the scaled cloud contribution
				Contrib = Throughput * CloudResult.SigmaSDualHG;
				SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
				if (HitSample.Accept(SelectionWeight))
				{
					// stash this hit for next time
					FinalThroughput = Contrib / SelectionWeight;
					// We create a blend of SigmaHG (set to isotropic) and SigmaSDualHG (set to the original phase function) to match what raster does (lerp the eval)
					PackedPayload.SetVolumetricCallableShaderOutputSample(Ray.TMax, 0.0, P, 0.0, 1.0 - P, CloudResult.DualPhaseData, 0.0);
					PackedPayload.SetVolumetricCallableShaderOutputCloudDensityFactor(E); // so that future cloud shadow rays get attenuated
				}

				// Update scale factors for next iteration
				S *= S;
				P *= P;
				E *= E;
			}
			Throughput = FinalThroughput * HitSample.GetNormalization();
#endif
		}

		PackedPayload.SetVolumetricCallableShaderOutput(RandSeq, Throughput);
	}
	else
	{
		// Loop through combined density for sampling

		FRISContext HitSample = PackedPayload.GetVolumetricCallableShaderRISContext();

#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
		FCloudMaterialMSParams MS = (FCloudMaterialMSParams)0; 
		if (PathTracingCloudParameters.CloudShaderMultipleScatterApproxEnabled)
		{
			MS = CloudMaterialGetMSParams();
		}
#endif

		// zero out the payload so we can write our output to it
		PackedPayload = (FPackedPathTracingPayload) 0;
		PackedPayload.HitT = -1.0; // invalid sample until we are sure we got one

#define USE_VOXEL_TRAVERSAL	1

#if USE_VOXEL_TRAVERSAL
		// isect clip slabs, the AABB our clouds are enclosed in
		float3 Ro = mul(GetCloudClipBasis(), Ray.Origin);
		float3 Rd = mul(GetCloudClipBasis(), Ray.Direction);
		float3 InvRd = rcp(max(abs(Rd), 1e-6)) * select(Rd >= 0, 1.0, -1.0); // protect against division by 0

		// now that we know the ray length, lets march through the grid to see if we can clip away anything
		float2 P = Ro.xy + Ray.TMin * Rd.xy;
		int2 Idx = PositionToVoxel(P);
		float2 NextCrossingT = Ray.TMin + (VoxelToPosition(Idx + int2(Rd.xy >= 0)) - P) * InvRd.xy;
		float2 DeltaT = abs(CloudVoxelWidth * InvRd.xy);
		int2 Step = select_internal(Rd.xy >= 0, 1, -1);
		int2 Stop = select_internal(Rd.xy >= 0, CloudAccelMapResolution, -1);
#endif

		float3 VolumeRadiance = 0;
		float  VolumeAlpha = 0;
		float3 VolumeAlbedo = 0;
		// Limit number of steps to prevent timeouts
		// TODO: This biases the result! Is there a better way to limit the number of steps?
		for (int RaymarchStep = 0; RaymarchStep < PathTracingCloudParameters.MaxRaymarchSteps; RaymarchStep++)
		{
#if USE_VOXEL_TRAVERSAL
			// visit current voxel
			float4 Data = CloudAccelerationMap.Load(uint3(Idx.xy, 0));
			CloudDensity = Data.x;
			// because each cell has slabs, we can have at most 3 segments (empty, filled, empty) per grid cell
			float CellMinT = Ray.TMin;
			float CellMaxT = min3(Ray.TMax, NextCrossingT.x, NextCrossingT.y);
			float IntegrationMaxT = CellMaxT;
			if (CloudDensity > 0)
			{
				float SlabT0 = (Data.z * SKY_UNIT_TO_CM - Ro.z) * InvRd.z;
				float SlabT1 = (Data.w * SKY_UNIT_TO_CM - Ro.z) * InvRd.z;
				float InsideMinT = max(CellMinT, min(SlabT0, SlabT1));
				float InsideMaxT = min(CellMaxT, max(SlabT0, SlabT1));
				// do we still have a valid interval?
				if (InsideMinT < InsideMaxT)
				{
					// this interval must be going through at least 3 portions (some of which may be empty)
					// Figure out in which we are at the moment
					if (Ray.TMin < InsideMinT)
					{
						// we are outside the slab, take steps up to the slab boundary
						CloudDensity = 0;
						IntegrationMaxT = InsideMinT;
					} else if (Ray.TMin < InsideMaxT)
					{
						// we are inside the slab, take steps up to the end of the slab
						IntegrationMaxT = InsideMaxT;
					} else {
						// we are past the slab, take steps up to the cell boundary
						CloudDensity = 0;
					}
				}
				else
				{
					// we missed the slabs, so only keep the one segment, going through empty space
					CloudDensity = 0;
				}
			}
#else
			float IntegrationMaxT = Ray.TMax;
#endif

			float3 SigmaBar = CloudDensity;
			if (Flags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
			{
				SigmaBar += AtmosphereGetDensityBounds(Ray.Origin, Ray.Direction, Ray.TMin, Ray.TMax).SigmaMax;
			}
			if (Flags & PATH_TRACER_VOLUME_ENABLE_FOG)
			{
				SigmaBar += FogGetDensityBounds(Ray.Origin, Ray.Direction, Ray.TMin, Ray.TMax).SigmaMax;
			}

			// Interval is only crossing atmo/clouds/fog
			float RandValue = RandSeq.Get1D();
			float Distance = SampleSpectralTransmittance(RandValue, SigmaBar, PathThroughput);
			if (Distance < 0.0)
			{
				// no energy left
				break;
			}
			if (Ray.TMin + Distance < IntegrationMaxT)
			{
				float4 Evaluation = EvaluateSpectralTransmittanceHit(Distance, SigmaBar, PathThroughput);
				PathThroughput *= Evaluation.xyz;
				Ray.TMin += Distance;

				// find out how much volume exists at the current point
				float3 Ro = Ray.Origin;
				float3 Rd = Ray.Direction;
				float3 TranslatedWorldPos = Ro + Ray.TMin * Rd;
				FVolumeShadedResult Result = (FVolumeShadedResult)0; 
				MergeVolumeShadedResult(Result, CloudsGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_CLOUDS) != 0);
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
				FVolumeShadedResult CloudResult = Result; // capture just the Cloud contribution (including holdouts) so we can scale it for the MS approx
#endif
				if (Flags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
				{
					MergeVolumeShadedResult(Result, AtmosphereGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE) != 0);
				}
				if (Flags & PATH_TRACER_VOLUME_ENABLE_FOG)
				{
					MergeVolumeShadedResult(Result, FogGetDensity(TranslatedWorldPos), bIsCameraRay && (Flags & PATH_TRACER_VOLUME_HOLDOUT_FOG) != 0);
				}

				// clamp to make sure we never exceed the majorant (should not be the case, but need to avoid any possible numerical issues)
				float3 SigmaT = min(Result.SigmaT, SigmaBar);
				float3 SigmaN = SigmaBar - SigmaT;

				float3 SigmaH = min(Result.SigmaH, SigmaT);
				VolumeAlpha += Luminance(PathThroughput * (SigmaT - SigmaH));

				// If this ray wants to get candidate samples for scatter, process them here
				if (Flags & PATH_TRACER_VOLUME_CALLABLE_FLAGS_GET_SAMPLE)
				{
					float3 SigmaS = min(Result.SigmaSRayleigh + Result.SigmaSHG + Result.SigmaSDualHG, SigmaT);

					// accumulate a signal for the denoiser
					float3 Contrib = PathThroughput * SigmaS;
					VolumeAlbedo += Contrib;
					float SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
					if (HitSample.Accept(SelectionWeight))
					{
						// stash this hit for next time
						float3 PayloadThroughput = Contrib / SelectionWeight;
						const float3 RayleighWeight = select(SigmaS > 0.0, Result.SigmaSRayleigh / SigmaS, 0.0);
						const float3 HGWeight       = select(SigmaS > 0.0, Result.SigmaSHG       / SigmaS, 0.0);
						const float3 DualHGWeight   = select(SigmaS > 0.0, Result.SigmaSDualHG   / SigmaS, 0.0);
						PackedPayload.SetVolumetricCallableShaderOutputSample(Ray.TMin, RayleighWeight, HGWeight, Result.PhaseG, DualHGWeight, Result.DualPhaseData, PayloadThroughput);
						PackedPayload.SetVolumetricCallableShaderOutputCloudDensityFactor(1.0);
					}
#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
					// The material has enabled fake-multiple scattering. We can take this into account here by stochastically returning additional lobes
					// NOTE: This is not energy conserving! TODO: how to integrate with indirect scattering?
					
					float S = MS.ScattFactor, E = MS.ExtinFactor, P = MS.PhaseFactor;
					for (int ms = 1; ms <= MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT; ms++)
					{
						CloudResult.SigmaSDualHG *= S;

						// Propose a candidate hit with just the scaled cloud contribution
						Contrib = PathThroughput * CloudResult.SigmaSDualHG;
						SelectionWeight = max3(Contrib.x, Contrib.y, Contrib.z);
						if (HitSample.Accept(SelectionWeight))
						{
							// stash this hit for next time
							float3 PayloadThroughput = Contrib / SelectionWeight;
							// We create a blend of SigmaHG (set to isotropic) and SigmaSDualHG (set to the original phase function) to match what raster does (lerp the eval)
							PackedPayload.SetVolumetricCallableShaderOutputSample(Ray.TMin, 0.0, P, 0.0, 1.0 - P, CloudResult.DualPhaseData, PayloadThroughput);
							PackedPayload.SetVolumetricCallableShaderOutputCloudDensityFactor(E); // so that future cloud shadow rays get attenuated
						}

						// Update scale factors for next iteration
						S *= S;
						P *= P;
						E *= E;
					}
#endif

				}

				VolumeRadiance += Result.Emission * PathThroughput;
				// keep tracing through the volume
				PathThroughput *= SigmaN;
				PathThroughput *= LowThroughputClampFactor(PathThroughput);
				if (!any(PathThroughput > 0))
				{
					break;
				}
			}
			else
			{
				// update the path throughput, knowing that we escaped the medium
				// exit the ray marching loop
				float4 Evaluation = EvaluateSpectralTransmittanceMiss(IntegrationMaxT - Ray.TMin, SigmaBar, PathThroughput);
				PathThroughput *= Evaluation.xyz;

#if USE_VOXEL_TRAVERSAL
				Ray.TMin = IntegrationMaxT;
#else
				break;
#endif
			}

#if USE_VOXEL_TRAVERSAL
			// Advance to next grid cell if we have reached the end of the cell
			if (Ray.TMin == CellMaxT)
			{
				if (NextCrossingT.x < NextCrossingT.y)
				{
					// step x first
					Idx.x += Step.x;
					if (Idx.x == Stop.x || Ray.TMax < NextCrossingT.x)
					{
						break;
					}
					Ray.TMin = NextCrossingT.x;
					NextCrossingT.x += DeltaT.x;
				}
				else
				{
					Idx.y += Step.y;
					if (Idx.y == Stop.y || Ray.TMax < NextCrossingT.y)
					{
						break;
					}
					Ray.TMin = NextCrossingT.y;
					NextCrossingT.y += DeltaT.y;
				}
			}
#endif
		}

		// Write state back here
		PackedPayload.SetVolumetricCallableShaderOutputRadiance(VolumeRadiance, VolumeAlpha);
		PackedPayload.SetVolumetricCallableShaderOutput(RandSeq, PathThroughput, VolumeAlbedo);
		PackedPayload.SetVolumetricCallableShaderRISContext(HitSample);
	}
}
