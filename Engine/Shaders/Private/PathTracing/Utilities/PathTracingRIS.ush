// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Implementation of the core logic of Weighted Resampled Importance Sampling
// The idea is to simulate sampling a discrete pdf built of individually weighted samples

// This implementation uses the sample-stretching trick presented in:
//  Shinji Ogaki - "Vectorized Reservoir Sampling"
//  Siggraph Asia 2021 Technical Communications

struct FRISContext
{
	float WeightSum;
	float RandSample;

	bool Accept(float Weight)
	{
		WeightSum += Weight;
        float Sum = WeightSum > 0 ? WeightSum : 1; // Avoid changing RandSample if we haven't accumulated anything yet
        RandSample *= Sum;
        bool Keep = RandSample < Weight;           // Should we accept this sample?
        RandSample -= Keep ? 0 : Weight;           // Rescale RandSample for re-use
        RandSample /= Keep ? Weight : Sum - Weight;
        RandSample = min(RandSample, 0.99999994);  // Make sure we stay in [0,1)
        return Keep;
	}

	bool HasSample()
	{
		return WeightSum > 0;
	}

	float GetNormalization()
	{
		return WeightSum;
	}
};

FRISContext InitRISContext(float RandSample)
{
	FRISContext Context = (FRISContext)0;
	Context.RandSample = RandSample;
	return Context;
}
