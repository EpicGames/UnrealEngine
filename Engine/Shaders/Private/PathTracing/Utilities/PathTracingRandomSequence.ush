// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	PathTracingRandomSequence.ush: Reference path tracing
===============================================================================================*/

#pragma once

#include "../../MortonCode.ush"

#define RANDSEQ_PURERANDOM	0
#define RANDSEQ_OWENSOBOL	1
#define RANDSEQ_LATTICE     2

// Select a default if none was specified
#ifndef RANDSEQ
#define RANDSEQ			RANDSEQ_OWENSOBOL
#endif

#ifndef RANDSEQ_RANDOMIZED
#define RANDSEQ_RANDOMIZED		1 // 1 to randomize per pixel, 0 to share the same sequence in all pixels (useful to benchmark coherent sampling)
#endif

#ifndef RANDSEQ_ERROR_DIFFUSION
#define RANDSEQ_ERROR_DIFFUSION 1 // Enabled by default, only works when using RANDSEQ_OWENSOBOL or RANDSEQ_LATTICE
#endif

#ifndef RANDSEQ_UNROLL_SOBOL
#define RANDSEQ_UNROLL_SOBOL 1 // Use unrolled Sobol loop such that table can be inline constants.
#endif


#ifndef RANDSEQ_SFC_TEXTURE
#define RANDSEQ_SFC_TEXTURE 0 // 0 uses a Hilbert curve (ALU heavy) while 1 uses a precomputed texture (must be passed as a shader parameter)
#endif

// This hash mixes bits at the theoretical bias limit
uint PerfectIntegerHash(uint x)
{
	// https://nullprogram.com/blog/2018/07/31/
	// exact bias: 0.020888578919738908
	x ^= x >> 17;
	x *= 0xed5ad4bbu;
	x ^= x >> 11;
	x *= 0xac4c1b51u;
	x ^= x >> 15;
	x *= 0x31848babu;
	x ^= x >> 14;
	return x;
}

// High quality integer hash - this mixes bits almost perfectly
uint StrongIntegerHash(uint x)
{
	// From https://github.com/skeeto/hash-prospector
	// Current best hash in this form: https://github.com/skeeto/hash-prospector/issues/19#issuecomment-1105792898
	// bias = 0.10734781817103507
	x ^= x >> 16;
	x *= 0x21f0aaad;
	x ^= x >> 15;
	x *= 0xf35a2d97;
	x ^= x >> 15;
	return x;
}

// Base must be a prime number
float Halton(uint Index, uint Base)
{
	float r = 0.0, f = 1.0;
	float BaseInv = 1.0 / Base;
	while (Index > 0)
	{
		f *= BaseInv;
		r += f * (Index % Base);
		Index /= Base;
	}
	return r;
}

uint FastOwenScramblingCore(uint Index, uint Seed)
{
#if 0
	// reference implementation - scramble bits one at a time

	// loop below does not require pre-inverted bits, so undo the inversion assumed in the optimized case below
	Index = reversebits(Index);
	for (uint Mask = 1u << 31, Input = Index; Mask; Mask >>= 1) {
		Seed = PerfectIntegerHash(Seed); // randomize state
		Index ^= Seed & Mask;  // flip output (depending on state)
		Seed ^= Input & Mask;  // flip state  (depending on input)
	}
	return Index;
#else
	// This simple tweak appears to be nearly as good as the reference, while being cheaper to compute
	Index ^= Index * 0xe0705d72u;
	Index += Seed;
	Seed ^= Seed >> 16;
	Index *= Seed | 1;
#endif
	return Index;
}

uint FastOwenScrambling(uint Index, uint Seed)
{
	Index = FastOwenScramblingCore(Index, Seed);
	// Undo the reverse so that we get left-to-right scrambling
	// thereby emulating owen-scrambling
	return reversebits(Index);
}

#if RANDSEQ_SFC_TEXTURE == 1
Texture2D<uint> RandomSequenceSpaceFillingCurve;
// Given a point (X,Y) on the unit square [0,2^B)^2, return the index along a space filling curve in [0,2^2B)
// The space filling curve index is encoded in a texture which must be passed in as a shader parameter.

uint SFCInverse(uint X, uint Y, int NumBits)
{
	const uint Mask = (1u << NumBits) - 1;
	return RandomSequenceSpaceFillingCurve.Load(uint3(X & Mask, Y & Mask, 0));
}
#else
// Given a point (X,Y) on the unit square [0,2^B)^2, return the index along the hilbert curve in [0,2^2B)
// Because this function only loops over the bits it uses, there is no need to mask out bits beyond the
// bit count, the pattern will simply tile. This is meant to be a fallback if you don't want/need to pass in
// the space filling curve texture.
uint SFCInverse(uint X, uint Y, int NumBits)
{
	// https://github.com/hcs0/Hackers-Delight/blob/master/hilbert/hil_s_from_xy.c.txt
	uint HilbertIndex = 0, HilbertState = 0;
	for (int i = NumBits; i--;) {
		uint xi = (X >> i) & 1;
		uint yi = (Y >> i) & 1;
		uint Row = 8 * HilbertState + 4 * xi + 2 * yi;
		HilbertIndex = HilbertIndex * 4 + ((0x361E9CB4u >> Row) & 3);
		HilbertState = (0x8FE65831u >> Row) & 3;
	}
	return HilbertIndex;
}
#endif

uint EvolveSobolSeed(inout uint Seed)
{
	// LCG parameters from
	// "Computationally Easy, Spectrally Good Multipliers for Congruential Pseudorandom Number Generators"
	//   Guy Steele, Sebastiano Vigna
	//  Journal of Software: Practice and Experience, Volume 52, Issue 2, Feb 2022
	return Seed = Seed * 0x915f77f5u + 0x93d765ddu;
}

uint4 SamplerCore(uint SampleIndex, inout uint Seed)
{
#if RANDSEQ == RANDSEQ_PURERANDOM
	uint4 Result = uint4(StrongIntegerHash(Seed + 0),
						 StrongIntegerHash(Seed + 1),
						 StrongIntegerHash(Seed + 2),
						 StrongIntegerHash(Seed + 3));
	Seed += 4;
	return Result;
#elif RANDSEQ == RANDSEQ_OWENSOBOL
	// first scramble the index to decorelate from other 4-tuples
	uint SobolIndex = FastOwenScrambling(SampleIndex, EvolveSobolSeed(Seed));
	// now get Sobol' point from this index
	uint4 Result = uint4(SobolIndex, SobolIndex, 0, 0);
	// "SZ Sequences: Binary-Based (0,2^q)-Sequences"
	//  Abdalla G. M. Ahmed, Matt Pharr, Victor Ostromoukhov, Hui Huang
	// 	  https://dl.acm.org/doi/epdf/10.1145/3763272
	//    Siggraph Asia 2025
	// Optimized evaluation of Dimension 2 via Equation 42
	Result.z = ((SobolIndex >> 1) & 0x50550550u) ^
	           ((SobolIndex     ) & 0xaf5af5afu) ^
	           ((SobolIndex << 1) & 0xa0aa0aa0u);
	Result.z ^=  Result.z               << 16;
	Result.z ^= (Result.z & 0x00FF00FF) <<  8;
	Result.z ^= (Result.z & 0x0F0F0F0F) <<  4;
	Result.z = ((Result.z >> 1) & 0x50550550u) ^
	           ((Result.z     ) & 0x5fa5fa5fu) ^
	           ((Result.z << 1) & 0xa4ea4ea4u) ^
	           ((Result.z << 2) & 0xc84c84c8u) ^
	           ((Result.z << 3) & 0x08808808u);
	Result.zw = reversebits(Result.z);
   	// Result.yw can be computed without iteration by starting with Result.xz
	// "An Implementation Algorithm of 2D Sobol Sequence Fast, Elegant, and Compact"
	// Abdalla Ahmed, EGSR 2024
	// See listing (19) in the paper
	// The code is different here because we want the output to be bit-reversed, but
	// the methodology is the same
	Result.yw ^=  Result.yw               >> 16;
	Result.yw ^= (Result.yw & 0xFF00FF00) >>  8;
	Result.yw ^= (Result.yw & 0xF0F0F0F0) >>  4;
	Result.yw ^= (Result.yw & 0xCCCCCCCC) >>  2;
	Result.yw ^= (Result.yw & 0xAAAAAAAA) >>  1;
	// finally scramble the points to avoid structured artifacts
	Result.x = FastOwenScrambling(Result.x, EvolveSobolSeed(Seed));
	Result.y = FastOwenScrambling(Result.y, EvolveSobolSeed(Seed));
	Result.z = FastOwenScrambling(Result.z, EvolveSobolSeed(Seed));
	Result.w = FastOwenScrambling(Result.w, EvolveSobolSeed(Seed));
	return Result;
#elif RANDSEQ == RANDSEQ_LATTICE
	// Simplified algorithm using rank-1 lattice sequences. It avoids the extra call to reverse bits
	// Scrambling the _output_ does not appear to help, so we only need to scramble the index, leading to a very fast algorithm
	// Unfortunately, the resulting sequence does not have quite as good quality as Sobol in practice

	uint LatticeIndex = FastOwenScramblingCore(SampleIndex, EvolveSobolSeed(Seed));

	// Lattice parameters taken from:
	// Weighted compound integration rules with higher order convergence for all N
	// Fred J. Hickernell, Peter Kritzer, Frances Y. Kuo, Dirk Nuyens
	// Numerical Algorithms - February 2012
	return LatticeIndex * uint4(1, 364981, 245389, 97823);
#endif
}

/// A simple struct that encapsulates a random number sequence (extensible in dimension)
/// Once you create the struct, you can draw samples in the unit square (as either floats or bits) in any dimension from 1 to 4.
/// If you need more dimensions, you can repeat the calls, which will be guaranteed to be statistically independent from the previous
/// However the values from a single call (Get1D(), Get2D(), etc ...) should have nice stratification properties
/// If you need to draw N samples for a pixel, you need to create the struct N times.
struct FRandomSequence
{
	uint SampleIndex;		// index into the random sequence
	uint SampleSeed;		// changes as we draw samples to reflect the change in dimension

	// Return a random value in [0,2^32)^d
	// All functions rely on the 4D variant and we rely on compiler to optimize out the dead code
	// This also guarantees that the sequence will not "shift" if you change a call from one dimension to another because the seed will be changed in a consistent way in all cases
	uint  Get1DBits() { return SamplerCore(SampleIndex, SampleSeed).x; }
	uint2 Get2DBits() { return SamplerCore(SampleIndex, SampleSeed).xy; }
	uint3 Get3DBits() { return SamplerCore(SampleIndex, SampleSeed).xyz; }
	uint4 Get4DBits() { return SamplerCore(SampleIndex, SampleSeed); }

	// Convert the random value in [0,2^32)^d to a uniformly distributed float in [0,1)^d,
	// taking care not to skew the distribution due to the non-uniform spacing of floats.
	// Use the upper 24 bits to preserve the good properties of low-discrenpancy samplers.
	// The floating point constant is 2^-24 (would be more nicely expressed with hex-floats which are not yet supported in HLSL)
	// The implicit int to float cast here is guaranteed to be lossless because float can represent integers up to 2^24 exactly
	float  Get1D() { return (Get1DBits() >> 8) * 5.96046447754e-08; }
	float2 Get2D() { return (Get2DBits() >> 8) * 5.96046447754e-08; }
	float3 Get3D() { return (Get3DBits() >> 8) * 5.96046447754e-08; }
	float4 Get4D() { return (Get4DBits() >> 8) * 5.96046447754e-08; }

	// Split the path into Num entries. The RandomSequence object returned can be used to keep sampling along the "Index" branch of the split
	// Index must be a value in [0,Num), Num should be >0
	FRandomSequence Split(uint Index, uint Num);
};

// This initializes a multi-dimensional sampler for a particular pixel. The FrameIndex parameter should be set
// such that it changes per sample. See the function below when you know how many samples you plan on taking
// for a particular integral.
FRandomSequence RandomSequenceCreate(uint PositionSeed, uint FrameIndex)
{
	// optionally disable randomization per pixel so that all pixels follow the same path in primary sample space
#if RANDSEQ_RANDOMIZED == 0
	PositionSeed = 0;
#endif

	FRandomSequence RandSequence = (FRandomSequence) 0;

#if RANDSEQ == RANDSEQ_PURERANDOM
	RandSequence.SampleIndex = 0; // not used
	RandSequence.SampleSeed = StrongIntegerHash(PositionSeed + StrongIntegerHash(FrameIndex));
#elif RANDSEQ == RANDSEQ_OWENSOBOL || RANDSEQ == RANDSEQ_LATTICE
	// pre-compute bit reversal needed for FastOwenScrambling since this index doesn't change
	RandSequence.SampleIndex = reversebits(FrameIndex);
	// change seed to get a unique sequence per pixel
	RandSequence.SampleSeed  = StrongIntegerHash(PositionSeed);
#else
#error "Unknown random sequence chosen in path tracer"
#endif
	return RandSequence;
}


// This is an alternative initialization to the method above when you know how many samples you are planning on taking. This incorporates the 
// TimeSeed0 (this should usually be the frame number when using all samples in one frame, or should be the frame on which the first sample was
// taken if taking one sample per frame like the path tracer).
FRandomSequence RandomSequenceCreate(uint3 PixelCoordAndFrame, uint SampleIndex, uint MaxSamples)
{
	FRandomSequence RandSequence = (FRandomSequence) 0;

#if RANDSEQ_ERROR_DIFFUSION == 1 && (RANDSEQ == RANDSEQ_OWENSOBOL || RANDSEQ == RANDSEQ_LATTICE)
	// The core idea is to use a single sobol pattern across the screen using a space filling curve to correlate nearby pixels
	// This achieves a much better error diffusion than the random sequence assignment of the method above
	// This algorithm is covered in the following paper:
	//   "Screen-Space Blue-Noise Diffusion of Monte Carlo Sampling Error via Hierarchical Ordering of Pixels"
	//   Siggraph Asia 2020
	//   http://abdallagafar.com/publications/zsampler/
	// Improvements made in this version are: 
	//   - switching the z-order curve for a sierpinski curve
	//   - switching sobol points for owen-sobol points
	//   - recognizing that the hierarchical scrambling proposed in the original paper is equivalent to Owen-Scrambling of the index
	// Downsides of this method:
	//   - quality does not improve beyond the target sample count, making usage with adaptive sampling difficult
	//   - error-diffusion property is only achieved at the target sample count, earlier samples look random (sometimes slightly worse than random)

	uint TileID = SFCInverse(PixelCoordAndFrame.x, PixelCoordAndFrame.y, 8);

#if 0
	// Combine frame/sample index into the spatial index in a way that diffuses the error spatially and temporally
	// However this appears to be worse overall, averaging successive frames does not converge quickly
	// instead there are artifacts resulting from the space filling curve being explored in linear order.
	TileID ^= FastOwenScrambling(reversebits(PixelCoordAndFrame.z), 0xcafef00du);
#else
	TileID += PixelCoordAndFrame.z * 65536;
#endif

	// Combine pixel-level and sample-level bits into the sample index (visible structure will be hidden by owen scrambling of the index)
	RandSequence.SampleIndex = reversebits(TileID * MaxSamples + SampleIndex);

	// progressive encodings (these kind of work, but quality is much worse)
	//RandSequence.SampleIndex = reversebits(MortonEncode(uint2(TimeSeed0, TileID)) * MaxSamples + SampleIndex);
	//RandSequence.SampleIndex = reversebits((MortonCode3(MaxSamples * TimeSeed0 + SampleIndex) * 4) | (MortonCode3(Y) * 2) | MortonCode3(X));
	//RandSequence.SampleIndex = reversebits(MortonEncode(uint2(MortonEncode(uint2(X, Y)), SampleIndex)));

	RandSequence.SampleSeed = 0; // always use the same sequence
	return RandSequence;
#else
	// Error diffusion sampler is disabled, just use the simple init function instead
	return RandomSequenceCreate(PixelCoordAndFrame.x + PixelCoordAndFrame.y * 65536, PixelCoordAndFrame.z * MaxSamples + SampleIndex);
#endif
}

FRandomSequence FRandomSequence::Split(uint Index, uint Num)
{
	FRandomSequence SplitSequence;
#if RANDSEQ == RANDSEQ_PURERANDOM
	SplitSequence.SampleIndex = 0;
	SplitSequence.SampleSeed = StrongIntegerHash(SampleSeed * Num + Index);
#else
	// Undo the built-in reversebits, stretch the index and re-apply it
	SplitSequence.SampleSeed = SampleSeed;
	SplitSequence.SampleIndex = reversebits(reversebits(SampleIndex) * Num + Index);
#endif
	return SplitSequence;
}


// Legacy API - wrappers around the more convenient calls above, but kept for back-compatiblity
// Deprecated (5.7)
typedef FRandomSequence RandomSequence;
void RandomSequence_Initialize(inout RandomSequence RandSequence, uint PositionSeed, uint FrameIndex)
{
	RandSequence = RandomSequenceCreate(PositionSeed, FrameIndex);
}

void RandomSequence_Initialize(inout RandomSequence RandSequence, uint2 PixelCoord, uint SampleIndex, uint FrameIndex, uint MaxSamples)
{
	RandSequence = RandomSequenceCreate(uint3(PixelCoord, FrameIndex), SampleIndex, MaxSamples);
}

float RandomSequence_GenerateSample1D(inout RandomSequence RandSequence)
{
	return RandSequence.Get1D();
}

float2 RandomSequence_GenerateSample2D(inout RandomSequence RandSequence)
{
	return RandSequence.Get2D();
}

float3 RandomSequence_GenerateSample3D(inout RandomSequence RandSequence)
{
	return RandSequence.Get3D();
}

float4 RandomSequence_GenerateSample4D(inout RandomSequence RandSequence)
{
	return RandSequence.Get4D();
}
