// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "./PathTracingAtmosphereCommon.ush"
#include "../../DoubleFloat.ush"
#include "../../SkyAtmosphereCommon.ush"


#define PLANET_ISECT_USES_DOUBLE_WORD_MATH 1

// Shader parameters
// NOTE: Atmosphere is a global struct (not directly defined here)
#ifndef PlanetCenterTranslatedWorldHi
float3 PlanetCenterTranslatedWorldHi;
float3 PlanetCenterTranslatedWorldLo;
Texture2D<float3> AtmosphereOpticalDepthLUT;
SamplerState      AtmosphereOpticalDepthLUTSampler;
#endif

#if PLANET_ISECT_USES_DOUBLE_WORD_MATH

// intersect the planet using high precision
// returns: planet hit T, atmo hit min/max (or -1.0 for any if missed)
float3 RayPlanetIntersectPrecise(float3 Ro, float3 Rd)
{
	float3 Result = -1.0;

	const FDFVector3 Center = MakeDFVector3(PlanetCenterTranslatedWorldHi, PlanetCenterTranslatedWorldLo);
	const FDFScalar PlanetRadius = DFMultiply(View.SkyAtmosphereBottomRadiusKm, SKY_UNIT_TO_CM);
	const FDFScalar AtmoRadius   = DFMultiply(View.SkyAtmosphereTopRadiusKm   , SKY_UNIT_TO_CM);

	const FDFVector3 Co = DFSubtract(Center, Ro);
	const FDFScalar b = DFDot(Co, Rd);
	const FDFVector3 H = DFSubtract(Co, DFMultiply(Rd, b));
	const FDFScalar hd = DFLengthSqr(H);
	const FDFScalar AR2 = DFSqr(AtmoRadius);
	FDFScalar ha = DFSubtract(AR2, hd);
	if (DFGreater(ha, 0.0))
	{
		ha = DFSqrt(ha);
		FDFScalar q;
		if (DFGreater(b, 0.0))
			q = DFAdd(b, ha);
		else
			q = DFSubtract(b, ha);
		const float t0 = DFDemote(q);
		const float t1 = DFFastDivideDemote(DFSubtract(DFLengthSqr(Co), AR2), q);
		Result.y = min(t0, t1);
		Result.z = max(t0, t1);
	}
	const FDFScalar PR2 = DFSqr(PlanetRadius);
	FDFScalar hp = DFSubtract(PR2, hd);
	if (DFGreater(hp, 0.0))
	{
		hp = DFSqrt(hp);
		FDFScalar q;
		if (DFGreater(b, 0.0))
			q = DFAdd(b, hp);
		else
			q = DFSubtract(b, hp);
		const float t0 = DFDemote(q);
		const float t1 = DFFastDivideDemote(DFSubtract(DFLengthSqr(Co), PR2), q);
		Result.x = min(t0, t1);
	}
	return Result;
}

#else

// robust intersector with plain floats, still runs into trouble beyond 1000km or so
float3 RayPlanetIntersectPrecise(float3 Ro, float3 Rd)
{
	float3 Result = -1.0;
	float3 Center      = PlanetCenterTranslatedWorldHi + PlanetCenterTranslatedWorldLo;
	float PlanetRadius = View.SkyAtmosphereBottomRadiusKm * SKY_UNIT_TO_CM;
	float AtmoRadius   = View.SkyAtmosphereTopRadiusKm    * SKY_UNIT_TO_CM;

	float3 co = Center - Ro;
	float b = dot(co, Rd);
	// "Precision Improvements for Ray / Sphere Intersection" - Ray Tracing Gems (2019)
	// https://link.springer.com/content/pdf/10.1007%2F978-1-4842-4427-2_7.pdf
	// NOTE: we assume the incoming ray direction is normalized
	float hd = length2(co - b * Rd);
	float ha = AtmoRadius * AtmoRadius - hd;
	if (ha > 0)
	{
		ha = sqrt(ha);
		float q = b > 0.0 ? b + ha : b - ha;
		float t0 = q;
		float t1 = (length2(co) - AtmoRadius * AtmoRadius) / q;
		Result.y = min(t0, t1);
		Result.z = max(t0, t1);
	}
	float hp = PlanetRadius * PlanetRadius - hd;
	if (hp > 0)
	{
		hp = sqrt(hp);
		float q = b > 0.0 ? b + hp : b - hp;
		float t0 = q;
		float t1 = (length2(co) - PlanetRadius * PlanetRadius) / q;
		Result.x = min(t0, t1);
	}

	return Result;
}

#endif

#if CLOUD_LAYER_PIXEL_SHADER == 0

FVolumeIntersection AtmosphereIntersect(float3 Origin, float3 Direction, float TMin, float TMax)
{
	float3 Result = RayPlanetIntersectPrecise(Origin, Direction);
	// clip to provided ray interval
	return CreateVolumeIntersection(
		max(TMin, Result.y),
		min(TMax, Result.z),
		Result.x > TMin && Result.x < TMax ? Result.x : -1.0);
}

FPackedPathTracingPayload AtmosphereGetBlockerHit(float3 Origin, float3 Direction, float HitT, bool bIsCameraRay)
{
	float3 Center = PlanetCenterTranslatedWorldHi + PlanetCenterTranslatedWorldLo;
	float3 WorldP = Origin + HitT * Direction;
	float3 WorldN = normalize(WorldP - Center);

	FPathTracingPayload Payload = (FPathTracingPayload)0;
	Payload.SetBaseColor(Atmosphere.GroundAlbedo.rgb);
	Payload.BSDFOpacity = 1.0;
	Payload.WorldNormal = Payload.WorldGeoNormal = Payload.WorldSmoothNormal = WorldN;
	Payload.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_LAMBERT; // defaults to diffuse
	Payload.HitT = HitT;
	Payload.PrimitiveLightingChannelMask = 7; // visible to all lights
	Payload.SetFrontFace();
	if (bIsCameraRay && (VolumeFlags & PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE) != 0)
	{
		Payload.SetHoldout();
		Payload.SetBaseColor(0);
	}
	return PackPathTracingPayload(Payload);
}

#endif

struct FSphericalPlanetParameters {
	float HeightKm;         // Distance from planet center (in Km)
	float ZenithCosAngle;   // angle between the ray and the planet normal (+1 goes up towards space, -1 points down toward the ground)

	FSphericalPlanetParameters GetAt(float TKm)
	{
		// work out the ViewHeight / CosAngle of the specified point along the ray (in SkyUnits)
		const float TOverH = TKm / HeightKm;
		const float RelativeHeight = sqrt(mad(mad(2.0f, ZenithCosAngle, TOverH), TOverH, 1.0f));
		FSphericalPlanetParameters Result;
		Result.HeightKm = HeightKm * RelativeHeight;
		Result.ZenithCosAngle = (TOverH + ZenithCosAngle) / RelativeHeight;
		return Result;
	}

	float GetHorizonCosine()
	{
		return ComputeHorizonCosine(HeightKm, Atmosphere.BottomRadiusKm);
	}
};

FSphericalPlanetParameters GetSphericalPlanetParameters(float3 Origin, float3 Direction)
{
	FSphericalPlanetParameters Result = (FSphericalPlanetParameters)0;
#if PLANET_ISECT_USES_DOUBLE_WORD_MATH
	// Compute the origin's height/angle with high precision since it involves distances to the planet center
	const FDFVector3 Center = MakeDFVector3(PlanetCenterTranslatedWorldHi, PlanetCenterTranslatedWorldLo);

	const FDFVector3 OC = DFMultiply(DFSubtract(Origin, Center), CM_TO_SKY_UNIT);

	const FDFScalar HeightKm = DFLength(OC);
	Result.HeightKm = DFDemote(HeightKm);
	Result.ZenithCosAngle = DFFastDivideDemote(DFDot(OC, Direction), HeightKm);
#else
	// Get the ViewHeight / CosAngle of the origin
	const float3 PosFromCenter = Origin - (PlanetCenterTranslatedWorldHi + PlanetCenterTranslatedWorldLo);
	Result.HeightKm = length(PosFromCenter * CM_TO_SKY_UNIT);
	Result.ZenithCosAngle = dot(normalize(PosFromCenter), Direction);
#endif
	return Result;
}

FVolumeDensityBounds AtmosphereGetDensityBounds(float3 Origin, float3 Direction, float TMin, float TMax)
{
#if 0
	// conservative global bound for validation
	const float ta = Atmosphere.AbsorptionDensity0LayerWidth;
	const float tv = Atmosphere.AbsorptionDensity0LinearTerm * ta + Atmosphere.AbsorptionDensity0ConstantTerm;
	float3 SigmaMax = (Atmosphere.MieExtinction.rgb + Atmosphere.RayleighScattering.rgb + tv * Atmosphere.AbsorptionExtinction.rgb) * CM_TO_SKY_UNIT;
	return CreateVolumeDensityBound(0.0, SigmaMax);
#else
	// optimized bounds for the current ray
	FSphericalPlanetParameters Params = GetSphericalPlanetParameters(Origin, Direction);

	const float Ha = Params.GetAt(TMin * CM_TO_SKY_UNIT).HeightKm;
	const float Hb = Params.GetAt(TMax * CM_TO_SKY_UNIT).HeightKm;
	// one of the two end-points has to be the closest
	float HLo = min(Ha, Hb);
	float HHi = max(Ha, Hb);
	// compute the closest point to the center along the ray
	float TClosest = -Params.HeightKm * Params.ZenithCosAngle * SKY_UNIT_TO_CM;
	// see if it falls within our ray segment
	if (TClosest > TMin && TClosest < TMax)
	{
		// if so, we can reduce our lower bound a bit more
		HLo = min(HLo, Params.HeightKm * sqrt(saturate(1 - Params.ZenithCosAngle * Params.ZenithCosAngle)));
	}

	// now compute density bounds for each component of the atmosphere
	// exponential portion is easy since this is a stricly decreasing function of altitude
	const float2 SampleHeight = max(0.0, float2(HLo, HHi) - Atmosphere.BottomRadiusKm);
	const float2 DensityMie = exp(Atmosphere.MieDensityExpScale      * SampleHeight.yx); // flip height lo/hi because function is decreasing
	const float2 DensityRay = exp(Atmosphere.RayleighDensityExpScale * SampleHeight.yx);

	// Ozone is a bit trickier -- first eval both ends
	float2 DensityOzo = float2(
		SampleHeight.x < Atmosphere.AbsorptionDensity0LayerWidth ?
			saturate(Atmosphere.AbsorptionDensity0LinearTerm * SampleHeight.x + Atmosphere.AbsorptionDensity0ConstantTerm) :
			saturate(Atmosphere.AbsorptionDensity1LinearTerm * SampleHeight.x + Atmosphere.AbsorptionDensity1ConstantTerm),
		SampleHeight.y < Atmosphere.AbsorptionDensity0LayerWidth ?
			saturate(Atmosphere.AbsorptionDensity0LinearTerm * SampleHeight.y + Atmosphere.AbsorptionDensity0ConstantTerm) :
			saturate(Atmosphere.AbsorptionDensity1LinearTerm * SampleHeight.y + Atmosphere.AbsorptionDensity1ConstantTerm)
	);
	if (DensityOzo.x > DensityOzo.y)
		DensityOzo = DensityOzo.yx; // flip if wrong way around
	if (SampleHeight.x < Atmosphere.AbsorptionDensity0LayerWidth && Atmosphere.AbsorptionDensity0LayerWidth < SampleHeight.y)
	{
		// height range includes the peak -- bump max to peak
		const float ta = Atmosphere.AbsorptionDensity0LayerWidth;
		const float tv = Atmosphere.AbsorptionDensity0LinearTerm * ta + Atmosphere.AbsorptionDensity0ConstantTerm;
		DensityOzo.y = tv;
	}
	return CreateVolumeDensityBound(
		(DensityMie.x * Atmosphere.MieExtinction.rgb + DensityRay.x * Atmosphere.RayleighScattering.rgb + DensityOzo.x * Atmosphere.AbsorptionExtinction.rgb) * CM_TO_SKY_UNIT,
		(DensityMie.y * Atmosphere.MieExtinction.rgb + DensityRay.y * Atmosphere.RayleighScattering.rgb + DensityOzo.y * Atmosphere.AbsorptionExtinction.rgb) * CM_TO_SKY_UNIT
	);
#endif
}

FVolumeShadedResult AtmosphereGetDensity(float3 TranslatedWorldPos)
{
#if 0 //PLANET_ISECT_USES_DOUBLE_WORD_MATH
	// Extra precision here doesn't seem to matter too much -- probably because density function is very smooth
	const float H = GetSphericalPlanetParameters(TranslatedWorldPos, float3(0, 0, 1)).HeightKm;
	MediumSampleRGB Medium = SampleAtmosphereMediumRGB(float3(0.0, 0.0, H));
#else
	float3 PosFromCenter = TranslatedWorldPos - (PlanetCenterTranslatedWorldHi + PlanetCenterTranslatedWorldLo);
	MediumSampleRGB Medium = SampleAtmosphereMediumRGB(PosFromCenter * CM_TO_SKY_UNIT);
#endif
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT         = Medium.Extinction    * CM_TO_SKY_UNIT;
	Result.SigmaSRayleigh = Medium.ScatteringRay * CM_TO_SKY_UNIT;
	Result.SigmaSHG       = Medium.ScatteringMie * CM_TO_SKY_UNIT;
	Result.PhaseG         = Atmosphere.MiePhaseG;
	return Result;
}

#if CLOUD_LAYER_PIXEL_SHADER == 0

float3 AtmosphereGetOpticalDensity(FSphericalPlanetParameters Params)
{
	float3 Tau = 0;
	if (Params.HeightKm < Atmosphere.BottomRadiusKm)
	{
		// Starting below ground level, need to compute transmittance through homogeneous region first
		// TODO: merge this into the LUT somehow (requires a new parameterization)

		// Ground level extinction, applies to entire planet core
		// We keep this value in reciprocal sky units because we measure the distance in sky units as well (canceling out to unitless optical depth)
		float3 SigmaT = SampleAtmosphereMediumRGB(float3(0, 0, 0)).Extinction;

		const float C = Params.ZenithCosAngle;
		const float H = Params.HeightKm;
		const float B = Atmosphere.BottomRadiusKm;
		const float DistanceToGround = -C * H + sqrt(max(B * B + H * H * (C * C - 1.0f), 0.0f));
		Tau = DistanceToGround * SigmaT;

		// Get remaining density from the point on the ground
		Params = Params.GetAt(DistanceToGround);
	}
	float2 UV = ToAtmosphereOpticalDepthLUTUV(Params.HeightKm, Params.ZenithCosAngle, Atmosphere.BottomRadiusKm, Atmosphere.TopRadiusKm);
	return Tau + AtmosphereOpticalDepthLUT.SampleLevel(AtmosphereOpticalDepthLUTSampler, UV, 0);
}

float3 AtmosphereGetTransmittance(float3 Origin, float3 Direction, float TMin, float TMax)
{
	FSphericalPlanetParameters Params = GetSphericalPlanetParameters(Origin, Direction);
	// now lookup the optical density LUT from each point (integral up to the farthest possible point)
	float3 DensityA = AtmosphereGetOpticalDensity(Params.GetAt(TMin * CM_TO_SKY_UNIT));
	float3 DensityB = AtmosphereGetOpticalDensity(Params.GetAt(TMax * CM_TO_SKY_UNIT));
	// get density for just this segment by removing double-counted density
	float3 Tau = max(DensityA - DensityB, 0.0);
	// compute transmittance
	return exp(-Tau);
}

#endif