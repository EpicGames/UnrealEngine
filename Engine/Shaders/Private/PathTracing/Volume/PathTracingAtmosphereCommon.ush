// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Atmosphere function that are independent of any shader parameters

float Gain(float x, float k)
{
	float a = 1 - 2 * x;
	return 0.5 - 0.5 * sign(a) * pow(abs(a), k);
}

float ComputeHorizonCosine(float ViewHeight, float BotRadius)
{
	const float R = BotRadius / ViewHeight;
	return -sqrt(saturate(1.0 - R * R));
}

float2 ToAtmosphereOpticalDepthLUTUV(float Height, float Cosine, float BotRadius, float TopRadius)
{
	const float HorizonCosine = ComputeHorizonCosine(Height, BotRadius);

	const float Q = 0.5; // line up horizon line to u=Q
	const float2 ab = Cosine >= HorizonCosine
		? float2((HorizonCosine - 1) / (Q - 1), (Q - HorizonCosine) / (Q - 1)) // above horizon
		: float2((1 + HorizonCosine) / Q, -1);                                 // below horizon
	
	float2 UV = float2((Cosine - ab.y) / ab.x, sqrt(saturate((Height - BotRadius) / (TopRadius - BotRadius))));
	// un-squish
	const float Sq = 2.0;
	UV.x = Gain(UV.x, 1.0 / Sq);
	return UV;
}

float2 FromAtmosphereOpticalDepthLUTUV(float2 UV, float BotRadius, float TopRadius)
{
	const float Height = lerp(BotRadius, TopRadius, Pow2(UV.y)); // more resolution toward the ground

	// from this height, at what cosine is the horizon line?
	const float HorizonCosine = ComputeHorizonCosine(Height, BotRadius);

	// squish around 0.5 to increase resolution near the horizon
	const float Sq = 2.0;
	const float U = Gain(UV.x, Sq);

	const float Q = 0.5; // line up horizon line to u=Q
	const float2 ab = U > Q
		? float2((HorizonCosine - 1) / (Q - 1), (Q - HorizonCosine) / (Q - 1)) // above horizon
		: float2((1 + HorizonCosine) / Q, -1);                                 // below horizon
	const float Cosine = ab.x * U + ab.y;

	return float2(Height, Cosine);
}
