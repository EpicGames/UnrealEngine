// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "./PathTracingAtmosphereCommon.ush"
#include "./PathTracingCloudsCommon.ush"


Texture2D<float4> CloudAccelerationMap;
SamplerState CloudAccelerationMapSampler;

// intersect the cloud layer with high precision
float4 RayCloudIntersectPrecise(float3 Ro, float3 Rd)
{
	float4 Result = -1.0;

	const FDFVector3 Center = MakeDFVector3(PlanetCenterTranslatedWorldHi, PlanetCenterTranslatedWorldLo);
	const FDFScalar CloudTopRadius = DFMultiply(CloudLayerTopKm, SKY_UNIT_TO_CM);
	const FDFScalar CloudBotRadius = DFMultiply(CloudLayerBotKm, SKY_UNIT_TO_CM);


	const FDFVector3 Co = DFSubtract(Center, Ro);
	const FDFScalar b = DFDot(Co, Rd);
	const FDFVector3 H = DFSubtract(Co, DFMultiply(Rd, b));
	const FDFScalar hd = DFLengthSqr(H);
	const FDFScalar TR2 = DFSqr(CloudTopRadius);
	FDFScalar ht = DFSubtract(TR2, hd);

	if (DFGreater(ht, 0.0))
	{
		ht = DFSqrt(ht);
		FDFScalar q;
		if (DFGreater(b, 0.0))
			q = DFAdd(b, ht);
		else
			q = DFSubtract(b, ht);
		const float t0 = DFDemote(q);
		const float t1 = DFFastDivideDemote(DFSubtract(DFLengthSqr(Co), TR2), q);
		Result.x = min(t0, t1);
		Result.y = max(t0, t1);
	}
	const FDFScalar BR2 = DFSqr(CloudBotRadius);
	FDFScalar hb = DFSubtract(BR2, hd);
	if (DFGreater(hb, 0.0))
	{
		hb = DFSqrt(hb);
		FDFScalar q;
		if (DFGreater(b, 0.0))
			q = DFAdd(b, hb);
		else
			q = DFSubtract(b, hb);
		const float t0 = DFDemote(q);
		const float t1 = DFFastDivideDemote(DFSubtract(DFLengthSqr(Co), BR2), q);
		Result.z = min(t0, t1);
		Result.w = max(t0, t1);
	}

	return Result;
}

FVolumeIntersection CloudsIntersect(float3 Origin, float3 Direction, float TMin, float TMax, float PathRoughness)
{
	if (PathRoughness > CloudRoughnessCutoff)
	{
		return CreateEmptyVolumeIntersection();
	}
	float4 Result = RayCloudIntersectPrecise(Origin, Direction);
	float VolumeTMin = max(Result.x, TMin);
	float VolumeTMax = min(Result.y, TMax);
	if (VolumeTMin < VolumeTMax)
	{
		// we intersected the top layer of clouds, refine to see if we hit the bottom layer
		if (Result.z > 0)
		{
			// bottom layer start in front - act as the back side of our interval
			VolumeTMax = min(Result.z, VolumeTMax);
		}
		else
		{
			// we are inside the bottom layer, or missed it entirely - clip the front segment
			VolumeTMin = max(VolumeTMin, Result.w);
		}

		// isect clip slabs, the AABB our clouds are enclosed in
		float3 Ro = mul(GetCloudClipBasis(), Origin);
		float3 Rd = mul(GetCloudClipBasis(), Direction);
		float3 InvRd = rcp(max(abs(Rd), 1e-6)) * select(Rd >= 0, 1.0, -1.0); // protect against division by 0
		float2 Slab0 = (-Ro.xy - CloudClipDistKm * SKY_UNIT_TO_CM) * InvRd.xy;
		float2 Slab1 = (-Ro.xy + CloudClipDistKm * SKY_UNIT_TO_CM) * InvRd.xy;
		VolumeTMin = max(VolumeTMin, max(min(Slab0.x, Slab1.x), min(Slab0.y, Slab1.y)));
		VolumeTMax = min(VolumeTMax, min(max(Slab0.x, Slab1.x), max(Slab0.y, Slab1.y)));
		if (VolumeTMin >= VolumeTMax)
			return CreateEmptyVolumeIntersection();

		VolumeTMax = min(VolumeTMax, VolumeTMin + CloudTracingMaxDistance);

#if 1 // use cloud accel map to tighten intersection bounds

		// now that we know the ray length, lets march through the grid to see if we can clip away anything
		float2 P = Ro.xy + VolumeTMin * Rd.xy;
		int2 Idx = PositionToVoxel(P);
		float2 NextCrossingT = VolumeTMin + (VoxelToPosition(Idx + int2(Rd.xy >= 0)) - P) * InvRd.xy;
		float2 DeltaT = abs(CloudVoxelWidth * InvRd.xy);
		int2 Step = select(Rd.xy >= 0, 1, -1);
		int2 Stop = select(Rd.xy >= 0, CloudAccelMapResolution, -1);
		
		// start with an empty interval
		float ResultTMin = VolumeTMax;
		float ResultTMax = VolumeTMin;
		float MaxDensity = 0;
		for (;;)
		{
			// visit current voxel
			float4 Data = CloudAccelerationMap.Load(uint3(Idx.xy, 0));
			bool bHasData = Data.x > 0;
			if (bHasData)
			{
				float CellMinT = VolumeTMin;
				float CellMaxT = min3(VolumeTMax, NextCrossingT.x, NextCrossingT.y);

				float SlabT0 = (Data.z * SKY_UNIT_TO_CM - Ro.z) * InvRd.z;
				float SlabT1 = (Data.w * SKY_UNIT_TO_CM - Ro.z) * InvRd.z;

				CellMinT = max(CellMinT, min(SlabT0, SlabT1));
				CellMaxT = min(CellMaxT, max(SlabT0, SlabT1));
				// do we still have a valid interval?
				if (CellMinT < CellMaxT)
				{
					// expand result bounds
					ResultTMin = min(ResultTMin, CellMinT);
					ResultTMax = max(ResultTMax, CellMaxT);
					MaxDensity = max(Data.x, MaxDensity);
				}
			}

			if (NextCrossingT.x < NextCrossingT.y)
			{
				// step x first
				Idx.x += Step.x;
				if (Idx.x == Stop.x || VolumeTMax < NextCrossingT.x)
				{
					break;
				}
				VolumeTMin = NextCrossingT.x;
				NextCrossingT.x += DeltaT.x;
			}
			else
			{
				Idx.y += Step.y;
				if (Idx.y == Stop.y || VolumeTMax < NextCrossingT.y)
				{
					break;
				}
				VolumeTMin = NextCrossingT.y;
				NextCrossingT.y += DeltaT.y;
			}
		}
		return CreateVolumeIntersectionWithDensity(ResultTMin, ResultTMax, MaxDensity);
#else
		// if we don't have the cloud accel map, we can't know the upper bound on density...
		return CreateVolumeIntersectionWithDensity(VolumeTMin, VolumeTMax, CM_TO_SKY_UNIT);
#endif
	}
	return CreateEmptyVolumeIntersection();
}
