// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../PathTracingCommon.ush"
#include "../../SkyAtmosphereCommon.ush"

#ifndef CloudClipX
float3 CloudClipX;
float3 CloudClipY;
float3 CloudClipZ;

float3 CloudClipCenterKm;

float CloudLayerBotKm;
float CloudLayerTopKm;
float CloudClipDistKm;
float CloudClipRadiusKm;

float CloudTracingMaxDistance;
float CloudRoughnessCutoff;
float CloudVoxelWidth;
float CloudInvVoxelWidth;
int CloudAccelMapResolution;
int CloudCallableShaderId;
#endif

float3x3 GetCloudClipBasis()
{
	return float3x3(CloudClipX, CloudClipY, CloudClipZ);
}

float GetLowestCloudZ()
{
	return sqrt(max(CloudLayerBotKm * CloudLayerBotKm - CloudClipDistKm * CloudClipDistKm * 2.0, 0.0));
}

float GetFarthestCornerDist()
{
	return sqrt(2 * CloudClipDistKm * CloudClipDistKm + max(Square(GetLowestCloudZ() - CloudClipRadiusKm), Square(CloudLayerTopKm - CloudClipRadiusKm)));
}

float2 GetRadiusBounds()
{
	float2 RBounds = float2(0, GetFarthestCornerDist());
	float CloudLoKm = GetLowestCloudZ();
	if (CloudClipRadiusKm < CloudLoKm)
	{
		RBounds.x = CloudLoKm - CloudClipRadiusKm;
	}
	else if (CloudClipRadiusKm > CloudLayerTopKm)
	{
		RBounds.x = CloudClipRadiusKm - CloudLayerTopKm;
	}
	return RBounds;
}

int2 PositionToVoxel(float2 P)
{
	int2 Idx = int2((P + CloudClipDistKm * SKY_UNIT_TO_CM) * CloudInvVoxelWidth);
	return clamp(Idx, 0, CloudAccelMapResolution - 1);
}

float2 VoxelToPosition(int2 Idx)
{
	return (Idx * CloudVoxelWidth) - CloudClipDistKm * SKY_UNIT_TO_CM;
}
