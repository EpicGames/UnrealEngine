// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Functions related to evaluating volumetric cloud materials in the path tracer
// This code is for sharing between the cloud acceleration map building and the callable shader version of the cloud material

#include "../../Common.ush"
#if CLOUD_VISUALIZATION_SHADER == 0
#include "/Engine/Generated/Material.ush"
#endif
#include "PathTracingVolumeCommon.ush"
#include "PathTracingCloudsCommon.ush"
#include "../../DoubleFloat.ush"
#include "../../SkyAtmosphereCommon.ush"


#if MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL

#define MATVEC              float
#define ColorToMATVEC(C)    ((C).r)   // Assume all 3 floats are the same
#define MATVEC_Max(C)       (C)
#define MATVEC_Min(C)       (C)
#define MATVEC_Avg(C)       (C)

#else

#define MATVEC              float3
#define ColorToMATVEC(C)    ((C).rgb)
float MATVEC_Max(float3 C) { return max3(C.r, C.g, C.b); }
float MATVEC_Min(float3 C) { return min3(C.r, C.g, C.b); }
float MATVEC_Avg(float3 C) { return dot(C, 1.0f / 3.0f); }

#endif

// Simplified struct that represents what a CloudMaterial can return
struct FSampleCloudMaterialResult
{
	MATVEC Extinction;
	MATVEC Albedo;
	MATVEC Emission;
#if MATERIAL_VOLUMETRIC_ADVANCED
	float PhaseG1;
	float PhaseG2;
	float PhaseBlend;
#endif
};

#if MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT > 0
struct FCloudMaterialMSParams
{
	float ScattFactor;
	float ExtinFactor;
	float PhaseFactor;
};

FCloudMaterialMSParams CloudMaterialGetMSParams()
{
	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	float4 SvPosition = 0; // TODO: should initialize this from the ray somehow?
	FPixelMaterialInputs PixelMaterialInputs = (FPixelMaterialInputs)0;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, true);

	FCloudMaterialMSParams Result = (FCloudMaterialMSParams)0;

	#if MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION
	Result.ScattFactor = saturate(GetVolumetricAdvancedMaterialOutput3(MaterialParameters));
	#else
	Result.ScattFactor = GetVolumetricAdvancedMaterialOutput3(MaterialParameters);
	#endif
	Result.ExtinFactor = saturate(GetVolumetricAdvancedMaterialOutput4(MaterialParameters));
	Result.PhaseFactor = saturate(GetVolumetricAdvancedMaterialOutput5(MaterialParameters));

	return Result;
}
#endif

#if CLOUD_VISUALIZATION_SHADER == 0
// For path tracing, we need to be able to evaluate the material starting from a single point in space
// In all contexts, we have a slightly more accurate Height already available so pass that in instead of measuring it from the position
FSampleCloudMaterialResult SampleCloudMaterial(FDFVector3 AbsoluteWorldPosition, float HeightKm, float CloudBotKm, float CloudTopKm)
{
	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	float4 SvPosition = 0; // TODO: could derive this from AbsoluteWorldPosition -- but is this needed? VolumetricCloud code doesn't update it per sample
	FPixelMaterialInputs PixelMaterialInputs = (FPixelMaterialInputs)0;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, true);

	MaterialParameters.WorldPosition_CamRelative = MaterialParameters.WorldPosition_NoOffsets_CamRelative = DFAddDemote(AbsoluteWorldPosition, PrimaryView.PreViewTranslation);
	MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);
	MaterialParameters.AbsoluteWorldPosition = MaterialParameters.WorldPosition_NoOffsets = AbsoluteWorldPosition;
	MaterialParameters.LWCData.AbsoluteWorldPosition = DFToWS(AbsoluteWorldPosition);
	MaterialParameters.CameraVector = normalize(PrimaryView.TranslatedWorldCameraOrigin - MaterialParameters.WorldPosition_CamRelative);

	MaterialParameters.CloudSampleAltitude = HeightKm * SKY_UNIT_TO_CM;
	MaterialParameters.CloudSampleAltitudeInLayer = (HeightKm - CloudBotKm) * SKY_UNIT_TO_CM;
	MaterialParameters.CloudSampleNormAltitudeInLayer = saturate((HeightKm - CloudBotKm) / (CloudTopKm - CloudBotKm));
	
	MaterialParameters.VolumeSampleConservativeDensity = 1.0f;
#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
	// Evaluate conservative density - this may be influenced by WorldPosition
	MaterialParameters.VolumeSampleConservativeDensity = GetVolumetricAdvancedMaterialOutput6(MaterialParameters);
	if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
	{
		return (FSampleCloudMaterialResult) 0;
	}
#endif

	CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

	FSampleCloudMaterialResult Result = (FSampleCloudMaterialResult)0;

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Result.Extinction = ColorToMATVEC(VOLUMETRICFOGCLOUD_EXTINCTION(BSDF));
	Result.Albedo     = ColorToMATVEC(VOLUMETRICFOGCLOUD_ALBEDO(BSDF));
	Result.Emission   = ColorToMATVEC(BSDF_GETEMISSIVE(BSDF));
		
#else
	#if !MATERIAL_SHADINGMODEL_UNLIT
	Result.Extinction = ColorToMATVEC(GetMaterialSubsurfaceDataRaw(PixelMaterialInputs));
	Result.Albedo     = ColorToMATVEC(GetMaterialBaseColor(PixelMaterialInputs).rgb * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz);
	#endif
	Result.Emission   = ColorToMATVEC(GetMaterialEmissiveRaw(PixelMaterialInputs));
#endif

	// Clamp to valid ranges
	Result.Extinction = clamp(Result.Extinction * CENTIMETER_TO_METER, 0.0f, 65000.0f);
	Result.Emission   = clamp(Result.Emission   * CENTIMETER_TO_METER, 0.0f, 65000.0f);
	Result.Albedo     = saturate(Result.Albedo);

#if MATERIAL_VOLUMETRIC_ADVANCED
	// Advanced node allows configuring the phase function
	Result.PhaseG1 = GetVolumetricAdvancedMaterialOutput0(MaterialParameters);
	Result.PhaseG2 = GetVolumetricAdvancedMaterialOutput1(MaterialParameters);
	Result.PhaseBlend = GetVolumetricAdvancedMaterialOutput2(MaterialParameters);
#endif

	return Result;
}
#endif
