// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// FogParameters
#ifndef FogDensity
float2 FogDensity;
float2 FogHeight;
float2 FogFalloff;
float4 FogAlbedo;
float FogPhaseG;
float2 FogCenter;
float FogMinZ;
float FogMaxZ;
float FogRadius;
float FogFalloffClamp;
#endif

FVolumeIntersection FogIntersect(float3 Origin, float3 Direction, float TMin, float TMax)
{
	// adapted from https://www.iquilezles.org/www/articles/intersectors/intersectors.htm
	// extended/simplified because we only need the entrance/exit distances
	float3 oc = Origin - float3(FogCenter, FogMinZ);

	float ba = FogMaxZ - FogMinZ;
	float baba = ba * ba;
	float bard = ba * Direction.z;
	float baoc = ba * oc.z;

	float k2 = baba - bard * bard;
	float k1 = baba * dot(oc, Direction) - baoc * bard;
	float k0 = baba * dot(oc, oc) - baoc * baoc - FogRadius * FogRadius * baba;

	float h = k1 * k1 - k2 * k0;
	if (h > 0.0)
	{
		float s = sign(k2);
		float2 TCyl = (-k1 + float2(-s, s) * sqrt(h)) / k2;
		float2 TCap = ((bard > 0.0 ? float2(0, baba) : float2(baba, 0)) - baoc) / bard;

		const float THitMin = max3(TCap.x, TCyl.x, TMin);
		const float THitMax = min3(TCap.y, TCyl.y, TMax);
		return CreateVolumeIntersection(THitMin, THitMax);
	}
	return CreateEmptyVolumeIntersection();
}

// returns the density for each bank of fog
float2 FogGetDualDensity(float Z)
{
	return FogDensity * exp2(-max((Z - FogHeight) * FogFalloff, FogFalloffClamp));
}

FVolumeDensityBounds FogGetDensityBounds(float3 Origin, float3 Direction, float TMin, float TMax)
{
#if 0
	// conservative default (reference)
	return CreateVolumeDensityBound(0, FogDensity.x + FogDensity.y);
#else
	// get bounds at each endpoint of the ray (density is a monotonic function, so just eval at both endpoints and sort)
	float2 DensityA = FogGetDualDensity(Origin.z + TMin * Direction.z);
	float2 DensityB = FogGetDualDensity(Origin.z + TMax * Direction.z);

	float2 DensityBounds = float2(
		DensityA.x + DensityA.y,
		DensityB.x + DensityB.y
	);
	// flip if in the wrong order
	if (DensityBounds.x > DensityBounds.y)
		DensityBounds = DensityBounds.yx;
	return CreateVolumeDensityBound(DensityBounds.x, DensityBounds.y);
#endif
}

// Given a point in world space, return the amount of volume and its scattering properties
FVolumeShadedResult FogGetDensity(float3 TranslatedWorldPos)
{
	FVolumeShadedResult Result = (FVolumeShadedResult)0;

	float2 Density = FogGetDualDensity(TranslatedWorldPos.z);
	Result.SigmaT = Density.x + Density.y;
	Result.SigmaSHG = FogAlbedo.xyz * Result.SigmaT;
	Result.PhaseG = FogPhaseG;
	return Result;
}

// Return the transmittance along a ray segment
float3 FogGetTransmittance(float3 Origin, float3 Direction, float TMin, float TMax)
{
	// Solve for intersection where (Z - FogHeight) * Falloff == FogFalloffClamp (or nearest point within slab)
	float2 FogClipZ = FogFalloffClamp / FogFalloff + FogHeight;
	float2 OH = Origin.z - FogHeight;
	float Dz = Direction.z;

	float2 Tau = 0.0;
	if (abs(Dz) < 1e-3)
	{
		// perfectly horizontal ray -- either go through all uniform or all varying, but either way density will be constant
		Tau = (TMax - TMin) * exp2(-max(FogFalloff * OH, FogFalloffClamp));
	}
	else
	{
		// figure out optical density through the uniform portion and varying portion respectively
		float2 TMid = clamp(-(Origin.z - FogClipZ) / Dz, TMin, TMax);
		float2 Ta = Dz > 0 ? TMid : TMin;
		float2 Tb = Dz > 0 ? TMax : TMid;
		// add varying portion (will be 0.0 from clamp of TMid if we don't actually enter the varying portion)
		Tau = exp2(-max(FogFalloff * (OH + Dz * Ta), FogFalloffClamp)) - exp2(-max(FogFalloff * (OH + Dz * Tb), FogFalloffClamp));
		Tau *= rcp(Dz * FogFalloff * log(2.0));

		// add uniform portion (will be 0.0 from clamp of TMid if we don't actually enter the uniform portion)
		Tau += (Dz > 0 ? TMid - TMin : TMax - TMid) * exp2(-FogFalloffClamp);
	}

	return exp(-dot(FogDensity, Tau));
}
