// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define USE_MEGAVOXEL 0
#define USE_CALLABLE_SHADERS 1
#define USE_MULTIPLE_VOLUMES 1

#define USE_PRESHADED_UNIFORM_BUFFER (PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE != USE_MEGAVOXEL)

#include "../../HeterogeneousVolumes/HeterogeneousVolumesTracingUtils.ush"
#include "../../HeterogeneousVolumes/HeterogeneousVolumesVoxelGridTraversal.ush"
#include "../Utilities/PathTracingRandomSequence.ush"
#include "PathTracingVolumeSampling.ush"
#include "PathTracingVolumeCommon.ush"
#include "PathTracingHeterogeneousVolumesForwardDecl.ush"
#include "PathTracingHeterogeneousVolumesMaterialCommon.ush"
#include "PathTracingHeterogeneousVolumesTrackingCommon.ush"
#include "PathTracingHeterogeneousVolumesPreshadingCommon.ush"

//
// Intersection API..
//
FVolumeIntersection HeterogeneousVolumesIntersectInternal(float3 Origin, float3 Direction, float TMin, float TMax)
{
	float3 WorldBoundsMin = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMin;
	float3 WorldBoundsMax = FrustumGridUniformBuffer.TopLevelGridWorldBoundsMax;
	if (OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		WorldBoundsMin = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin;
		WorldBoundsMax = OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax;
	}

	if (OrthoGridUniformBuffer.bUseOrthoGrid || FrustumGridUniformBuffer.bUseFrustumGrid)
	{
		float3 TranslatedWorldBoundsMin = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(WorldBoundsMin);
		float3 TranslatedWorldBoundsMax = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(WorldBoundsMax);

		float2 RayHitT = IntersectAABB(
			Origin,
			Direction,
			TMin,
			TMax,
			TranslatedWorldBoundsMin,
			TranslatedWorldBoundsMax
		);

		return CreateVolumeIntersection(RayHitT.x, RayHitT.y);
	}
	return CreateEmptyVolumeIntersection();
}

FVolumeIntersection HeterogeneousVolumesIntersect(float3 Origin, float3 Direction, float TMin, float TMax)
{
#if PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE == USE_MEGAVOXEL
	return HeterogeneousVolumesIntersectInternal(Origin, Direction, TMin, TMax);
#else
	return HeterogeneousVolumesIntersectPreshading(Origin, Direction, TMin, TMax);
#endif // PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE
}

//
// Density APIs..
//
FVolumeDensityBounds HeterogeneousVolumesGetDensityBoundsInternal(float3 Origin, float3 Direction, float TMin, float TMax)
{
	// Density bounds are iteratively discovered when executing the DDA on the majorant grid
	return CreateVolumeDensityBound(0, 0);
}

FVolumeDensityBounds HeterogeneousVolumesGetDensityBoundsPreshading(float3 Origin, float3 Direction, float TMin, float TMax)
{
	// Density bounds are iteratively discovered when executing the DDA on the majorant grid
	return CreateVolumeDensityBound(0, 1.0);
}

FVolumeDensityBounds HeterogeneousVolumesGetDensityBounds(float3 Origin, float3 Direction, float TMin, float TMax)
{
#if PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE == USE_MEGAVOXEL
	return HeterogeneousVolumesGetDensityBoundsInternal(Origin, Direction, TMin, TMax);
#else
	return HeterogeneousVolumesGetDensityBoundsPreshading(Origin, Direction, TMin, TMax);
#endif // PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE
}

FVolumeShadedResult GetOrthoVoxelGridDensity(float3 TranslatedWorldPos)
{
	float3 SigmaT = 0.0;
	float3 Albedo = 0.0;
	float3 Emission = 0.0;

	float3 TranslatedWorldBoundsMin = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin);
	float3 TranslatedWorldBoundsMax = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax);
	float3 TranslatedWorldBoundsExtent = TranslatedWorldBoundsMax - TranslatedWorldBoundsMin;
	float3 GridUV = (TranslatedWorldPos - TranslatedWorldBoundsMin) / TranslatedWorldBoundsExtent;
	if (all(GridUV >= 0.0) && all(GridUV <= 1.0))
	{
		float3 TopLevelVoxelPos = GridUV * OrthoGridUniformBuffer.TopLevelGridResolution;
		uint LinearTopLevelVoxelPos = GetLinearIndex(TopLevelVoxelPos, OrthoGridUniformBuffer.TopLevelGridResolution);

		if (OrthoGridUniformBuffer.bEnableIndirectionGrid)
		{
			FTopLevelGridData TopLevelGridData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelGridData))
			{
				uint IndirectionGridIndex = GetBottomLevelIndex(TopLevelGridData);
				uint3 IndirectionGridVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

				// Convert to Indirection voxel-space
				float3 IndirectionGridVoxelPos = frac(TopLevelVoxelPos) * IndirectionGridVoxelResolution;
				int3 IndirectionGridVoxelPosAsInt = clamp(IndirectionGridVoxelPos, 0, IndirectionGridVoxelResolution - 1);
				FTopLevelGridData IndirectionData = OrthoGridUniformBuffer.IndirectionGridBuffer[IndirectionGridIndex + MortonEncode3(IndirectionGridVoxelPosAsInt)];
				if (IsBottomLevelAllocated(IndirectionData))
				{
					uint BottomLevelIndex = GetBottomLevelIndex(IndirectionData);
					uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(IndirectionData);

					// Constant interpolation
					float3 BottomLevelVoxelPos = frac(IndirectionGridVoxelPos) * BottomLevelVoxelResolution;
					int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
					uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);
					SigmaT = GetExtinction(OrthoGridUniformBuffer.ExtinctionGridBuffer[LinearBottomLevelVoxelPos]);
					Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
					Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearBottomLevelVoxelPos]);
				}
			}
		}
		else
		{
			FTopLevelGridData TopLevelData = OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos];
			if (IsBottomLevelAllocated(TopLevelData))
			{
				uint BottomLevelIndex = GetBottomLevelIndex(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);
				uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(OrthoGridUniformBuffer.TopLevelGridBuffer[LinearTopLevelVoxelPos]);

				// Constant Interpolation
				float3 BottomLevelVoxelPos = frac(TopLevelVoxelPos) * BottomLevelVoxelResolution;
				int3 BottomLevelVoxelPosAsInt = clamp(BottomLevelVoxelPos, 0, BottomLevelVoxelResolution - 1);
				uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(BottomLevelVoxelPosAsInt);

				SigmaT = GetExtinction(OrthoGridUniformBuffer.ExtinctionGridBuffer[LinearBottomLevelVoxelPos]);
				Albedo = GetAlbedo(OrthoGridUniformBuffer.ScatteringGridBuffer[LinearBottomLevelVoxelPos]);
				Emission = GetEmission(OrthoGridUniformBuffer.EmissionGridBuffer[LinearBottomLevelVoxelPos]);
			}
		}
	}

	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT = SigmaT;
	Result.SigmaSHG = Albedo;
	Result.PhaseG = GetHeterogeneousVolumeCallableShaderPhaseG();
	Result.Emission = Emission;

	return Result;
}

FVolumeShadedResult GetFrustumVoxelGridDensity(float3 TranslatedWorldPos, inout bool bInFrustum)
{
	float3 SigmaT = 0.0;
	float3 Albedo = 0.0;
	float3 Emission = 0.0;

	// Convert TranslatedWorldPos to voxel space
	float3 WorldPos = DFHackToFloat(DFFastSubtract(TranslatedWorldPos, PrimaryView.PreViewTranslation));
	float3 ViewPos = mul(float4(WorldPos, 1), FrustumGridUniformBuffer.WorldToView).xyz;

	int3 VoxelDimensions = FrustumGridUniformBuffer.VoxelDimensions;
	float NearPlaneDepth = FrustumGridUniformBuffer.NearPlaneDepth;
	float FarPlaneDepth = FrustumGridUniformBuffer.FarPlaneDepth;
	float TanHalfFOV = FrustumGridUniformBuffer.TanHalfFOV;
	float3 VoxelPos = ViewToVoxel(ViewPos, VoxelDimensions, NearPlaneDepth, FarPlaneDepth, TanHalfFOV);

	bInFrustum = all(VoxelPos > 0) && all(VoxelPos < FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
	if (bInFrustum)
	{
		uint LinearTopLevelVoxelPos = GetLinearIndex(VoxelPos, FrustumGridUniformBuffer.TopLevelFroxelGridResolution);
		FTopLevelGridData TopLevelGridData = FrustumGridUniformBuffer.TopLevelFroxelGridBuffer[LinearTopLevelVoxelPos];
		if (IsBottomLevelAllocated(TopLevelGridData))
		{
			uint BottomLevelIndex = GetBottomLevelIndex(TopLevelGridData);
			uint3 BottomLevelVoxelResolution = GetBottomLevelVoxelResolution(TopLevelGridData);

			float3 BottomLevelVoxelPos = frac(VoxelPos) * BottomLevelVoxelResolution;
			uint LinearBottomLevelVoxelPos = BottomLevelIndex + MortonEncode3(uint3(BottomLevelVoxelPos));
			SigmaT = GetExtinction(FrustumGridUniformBuffer.ExtinctionFroxelGridBuffer[LinearBottomLevelVoxelPos]);
			Albedo = GetAlbedo(FrustumGridUniformBuffer.ScatteringFroxelGridBuffer[LinearBottomLevelVoxelPos]);
			Emission = GetEmission(FrustumGridUniformBuffer.EmissionFroxelGridBuffer[LinearBottomLevelVoxelPos]);
		}
	}

	// Return struct
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT = SigmaT;
	Result.SigmaSHG = Albedo;
	Result.PhaseG = GetHeterogeneousVolumeCallableShaderPhaseG();
	Result.Emission = Emission;
	return Result;
}

FVolumeShadedResult HeterogeneousVolumesGetDensityInternal(float3 TranslatedWorldPos)
{
	FVolumeShadedResult Result = (FVolumeShadedResult)0;

	bool bInFrustum = false;
	if (FrustumGridUniformBuffer.bUseFrustumGrid)
	{
		Result = GetFrustumVoxelGridDensity(TranslatedWorldPos, bInFrustum);
	}
	if (!bInFrustum && OrthoGridUniformBuffer.bUseOrthoGrid)
	{
		Result = GetOrthoVoxelGridDensity(TranslatedWorldPos);
	}
	return Result;
}

FVolumeShadedResult HeterogeneousVolumesGetDensity(float3 TranslatedWorldPos, int HeterogeneousVolumesIndex)
{
#if PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE == USE_MEGAVOXEL
	return HeterogeneousVolumesGetDensityInternal(TranslatedWorldPos);
#else
	return HeterogeneousVolumesGetDensityPreshading(TranslatedWorldPos, HeterogeneousVolumesIndex);
#endif // PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE
}

FVolumeTrackingResult HeterogeneousVolumesMajorantBasedTrackingInternal(float3 PathThroughput, float3 WorldRayOrigin, float3 WorldRayDirection, float WorldTMin, float WorldTMax, int bTransmittanceOnly, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence)
{
	FVolumeTrackingResult Result = CreateVolumeTrackingResult(WorldTMin);
	Result.Throughput = PathThroughput;

	float3 WorldRayBegin = WorldRayOrigin + WorldRayDirection * WorldTMin;
	float3 WorldRayEnd = WorldRayOrigin + WorldRayDirection * WorldTMax;
	float WorldRayTMax = length(WorldRayEnd - WorldRayBegin);

	// Transform to voxel-space
	float3 WorldBoundsMin = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMin);
	float3 WorldBoundsMax = GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(OrthoGridUniformBuffer.TopLevelGridWorldBoundsMax);
	float3 TopLevelGridWorldBoundsExtent = WorldBoundsMax - WorldBoundsMin;
	float3 VoxelRayBegin = (WorldRayBegin - WorldBoundsMin) / TopLevelGridWorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayEnd = (WorldRayEnd - WorldBoundsMin) / TopLevelGridWorldBoundsExtent * OrthoGridUniformBuffer.TopLevelGridResolution;
	float3 VoxelRayDirection = VoxelRayEnd - VoxelRayBegin;
	float VoxelRayTMin = 0.0;
	float VoxelRayTMax = length(VoxelRayDirection);
	float VoxelToWorldScale = WorldRayTMax / VoxelRayTMax;

	if (VoxelRayTMin >= VoxelRayTMax && !bTransmittanceOnly)
	{
		float LocalToWorldScale = 1.0;
		FVolumeTrackingResult Sample = HeterogeneousVolumesDeltaTracking(1, 0, WorldTMax - WorldTMin, LocalToWorldScale, OverlappingMajorant, RandSequence);
		if (Sample.Distance > 0.0)
		{
			Result = ConcatenateVolumeTrackingResult(Result, Sample);
		}
		return Result;
	}
	VoxelRayDirection /= VoxelRayTMax;

	// March majorant grid via DDA
	float3 VoxelRayDirectionInv = 1.0 / VoxelRayDirection;

	float VoxelRayMarchT = VoxelRayTMin;
	float3 VoxelRayMarchPos = VoxelRayBegin;

	float3 VoxelBoundsPos;
	VoxelBoundsPos.x = sign(VoxelRayDirection.x) > 0 ? floor(VoxelRayMarchPos.x) + 1 : ceil(VoxelRayMarchPos.x) - 1;
	VoxelBoundsPos.y = sign(VoxelRayDirection.y) > 0 ? floor(VoxelRayMarchPos.y) + 1 : ceil(VoxelRayMarchPos.y) - 1;
	VoxelBoundsPos.z = sign(VoxelRayDirection.z) > 0 ? floor(VoxelRayMarchPos.z) + 1 : ceil(VoxelRayMarchPos.z) - 1;

	float3 VoxelBoundsHitT = (VoxelBoundsPos - VoxelRayMarchPos) * VoxelRayDirectionInv;

	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (VoxelBoundsHitT.x <= Epsilon) VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
	if (VoxelBoundsHitT.y <= Epsilon) VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
	if (VoxelBoundsHitT.z <= Epsilon) VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);

	while ((VoxelRayMarchT < VoxelRayTMax) && any(Result.Throughput > Epsilon) && !Result.bIsCollision)
	{
		float VoxelRayMarchDeltaT = 0.0;
		if (VoxelBoundsHitT.x < VoxelBoundsHitT.y)
		{
			if (VoxelBoundsHitT.x < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.x - VoxelRayMarchT;
				VoxelBoundsHitT.x += abs(VoxelRayDirectionInv.x);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}
		else
		{
			if (VoxelBoundsHitT.y < VoxelBoundsHitT.z)
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.y - VoxelRayMarchT;
				VoxelBoundsHitT.y += abs(VoxelRayDirectionInv.y);
			}
			else
			{
				VoxelRayMarchDeltaT = VoxelBoundsHitT.z - VoxelRayMarchT;
				VoxelBoundsHitT.z += abs(VoxelRayDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (VoxelRayMarchT + VoxelRayMarchDeltaT > VoxelRayTMax)
		{
			VoxelRayMarchDeltaT = VoxelRayTMax - VoxelRayMarchT;
		}

		// Sample at the midpoint of the voxel ray
		float3 VoxelEndPos = VoxelRayMarchPos + VoxelRayDirection * VoxelRayMarchDeltaT;
		float3 VoxelSamplePos = (VoxelRayMarchPos + VoxelEndPos) * 0.5;
		uint VoxelSampleLinearPos = GetLinearIndex(VoxelSamplePos, OrthoGridUniformBuffer.TopLevelGridResolution);
		FMajorantData MajorantData = GetMajorantData(OrthoGridUniformBuffer.MajorantGridBuffer[VoxelSampleLinearPos]);
		MergeMajorantData(MajorantData, OverlappingMajorant);

		float WorldRayMarchT = VoxelRayMarchT * VoxelToWorldScale;
		float3 WorldTrackingOrigin = WorldRayBegin + WorldRayDirection * WorldRayMarchT;
		float WorldRayMarchDeltaT = VoxelRayMarchDeltaT * VoxelToWorldScale;

		FVolumeTrackingResult Sample;
		if (bTransmittanceOnly)
		{
			float LocalToWorldScale = 1.0;
			Sample = HeterogeneousVolumesRatioTracking(Result.Throughput, WorldTrackingOrigin, WorldRayDirection, WorldRayMarchDeltaT, LocalToWorldScale, MajorantData, RandSequence);
		}
		else
		{
			float LocalToWorldScale = 1.0;
			Sample = HeterogeneousVolumesDeltaTracking(1.0, 0.0, WorldRayMarchDeltaT, LocalToWorldScale, MajorantData, RandSequence);
		}

		if (Sample.Distance < 0)
		{
			return Sample;
		}

		Result = ConcatenateVolumeTrackingResult(Result, Sample);

		VoxelRayMarchT += VoxelRayMarchDeltaT;
		VoxelRayMarchPos = VoxelEndPos;
	}

	// Convert distances to world-units

	return Result;
}

//
// Distance Sampling API
//

FVolumeTrackingResult HeterogeneousVolumesSampleDistanceInternal(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence)
{
	bool bTransmittanceOnly = false;
	return HeterogeneousVolumesMajorantBasedTrackingInternal(PathThroughput, Origin, Direction, TMin, TMax, bTransmittanceOnly, OverlappingMajorant, RandSequence);
}

FVolumeTrackingResult HeterogeneousVolumesSampleDistance(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence)
{
#if PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE == USE_MEGAVOXEL
	return HeterogeneousVolumesSampleDistanceInternal(PathThroughput, Origin, Direction, TMin, TMax, OverlappingMajorant, RandSequence);
#else
	return HeterogeneousVolumesSampleDistancePreshading(PathThroughput, Origin, Direction, TMin, TMax, OverlappingMajorant, RandSequence);
#endif // PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE
}

//
// Transmittance API
//

float3 HeterogeneousVolumesGetTransmittanceInternal(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, inout FRandomSequence RandSequence)
{
	bool bTransmittanceOnly = true;
	FMajorantData EmptyOverlappingMajorant = CreateMajorantData(0.0f);
	return HeterogeneousVolumesMajorantBasedTrackingInternal(PathThroughput, Origin, Direction, TMin, TMax, bTransmittanceOnly, EmptyOverlappingMajorant, RandSequence).Throughput;
}

float3 HeterogeneousVolumesGetTransmittance(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, inout FRandomSequence RandSequence)
{
#if PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE == USE_MEGAVOXEL
	return HeterogeneousVolumesGetTransmittanceInternal(PathThroughput, Origin, Direction, TMin, TMax, RandSequence);
#else
	return HeterogeneousVolumesGetTransmittancePreshading(PathThroughput, Origin, Direction, TMin, TMax, RandSequence);
#endif // PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE
}
