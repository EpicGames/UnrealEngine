// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

FVolumeIntersection HeterogeneousVolumesIntersect(float3 Origin, float3 Direction, float TMin, float TMax);
FVolumeDensityBounds HeterogeneousVolumesGetDensityBounds(float3 Origin, float3 Direction, float TMin, float TMax);
FVolumeShadedResult HeterogeneousVolumesGetDensity(float3 TranslatedWorldPos, int CustomData);

struct FVolumeTrackingResult
{
	float3 Throughput;
	float Pdf;
	float Distance;
	float3 SigmaBar;
	bool bIsCollision;
	int HeterogeneousVolumesIndex;
};

FVolumeTrackingResult CreateVolumeTrackingResult(float Distance);
FVolumeTrackingResult ConcatenateVolumeTrackingResult(FVolumeTrackingResult lhs, FVolumeTrackingResult rhs);

FVolumeTrackingResult HeterogeneousVolumesSampleDistance(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence);
float3 HeterogeneousVolumesGetTransmittance(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, inout FRandomSequence RandSequence);

FVolumeTrackingResult HeterogeneousVolumesDeltaTracking(float3 PathThroughput, float LocalTMin, float LocalTMax, float LocalToWorldScale, FMajorantData MajorantData, inout FRandomSequence RandSequence);
FVolumeTrackingResult HeterogeneousVolumesRatioTracking(float3 PathThroughput, float3 TranslatedWorldOrigin, float3 WorldDirection, float LocalTMax, float LocalToWorldScale, FMajorantData MajorantData, inout FRandomSequence RandSequence);
