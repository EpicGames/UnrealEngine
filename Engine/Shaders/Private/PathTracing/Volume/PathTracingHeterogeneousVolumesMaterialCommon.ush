// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../../Common.ush"

// Transform

float3 GetHeterogeneousVolumeCallableShaderWorldPos(float3 TranslatedWorldPos)
{
	float3 WorldPos = DFFastLocalSubtractDemote(TranslatedWorldPos, PrimaryView.PreViewTranslation);
	return WorldPos;
}

float3 GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(float3 WorldPos)
{
	float3 TranslatedWorldPos = DFFastToTranslatedWorld(WorldPos, PrimaryView.PreViewTranslation);
	return TranslatedWorldPos;
}

// Shading

float GetHeterogeneousVolumeCallableShaderPhaseG()
{
	return 0.0;
}

// Intersection

void SetHeterogeneousVolumeCallableShaderHitT(
	float2 HitT,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.HitT = HitT.x;
	Payload.PackedData[0] = asuint(HitT.y);
}

float2 GetHeterogeneousVolumeCallableShaderHitT(
	FPackedPathTracingPayload Payload
)
{
	float2 HitT;
	HitT.x = Payload.HitT;
	HitT.y = asfloat(Payload.PackedData[0]);
	return HitT;
}

// Delta Tracking

void SetHeterogeneousVolumeCallableShaderRay(
	FRayDesc Ray,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.HitT = Ray.TMin;
	Payload.PackedData[0] = asuint(Ray.TMax);
	
	Payload.PackedData[1] = asuint(Ray.Origin.x);
	Payload.PackedData[2] = asuint(Ray.Origin.y);
	Payload.PackedData[3] = asuint(Ray.Origin.z);

	Payload.PackedData[4] = asuint(Ray.Direction.x);
	Payload.PackedData[5] = asuint(Ray.Direction.y);
	Payload.PackedData[6] = asuint(Ray.Direction.z);
}

FRayDesc GetHeterogeneousVolumeCallableShaderRay(
	FPackedPathTracingPayload Payload
)
{
	FRayDesc Ray;
	Ray.TMin = Payload.HitT;
	Ray.TMax = asfloat(Payload.PackedData[0]);

	Ray.Origin.x = asfloat(Payload.PackedData[1]);
	Ray.Origin.y = asfloat(Payload.PackedData[2]);
	Ray.Origin.z = asfloat(Payload.PackedData[3]);

	Ray.Direction.x = asfloat(Payload.PackedData[4]);
	Ray.Direction.y = asfloat(Payload.PackedData[5]);
	Ray.Direction.z = asfloat(Payload.PackedData[6]);
	return Ray;
}

void SetHeterogeneousVolumeCallableShaderRandomSequence(
	FRandomSequence RandSeq,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[7] = RandSeq.SampleIndex;
	Payload.PackedData[8] = RandSeq.SampleSeed;
}

FRandomSequence GetHeterogeneousVolumeCallableShaderRandomSequence(
	FPackedPathTracingPayload Payload
)
{
	FRandomSequence RandSeq;
	RandSeq.SampleIndex = Payload.PackedData[7];
	RandSeq.SampleSeed = Payload.PackedData[8];
	return RandSeq;
}

void SetHeterogeneousVolumeCallableShaderThroughput(
	float3 Throughput,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[9] = asuint(Throughput.x);
	Payload.PackedData[10] = asuint(Throughput.y);
	Payload.PackedData[11] = asuint(Throughput.z);
}

float3 GetHeterogeneousVolumeCallableShaderThroughput(
	FPackedPathTracingPayload Payload
)
{
	float3 Throughput = 0;
	Throughput.x = asfloat(Payload.PackedData[9]);
	Throughput.y = asfloat(Payload.PackedData[10]);
	Throughput.z = asfloat(Payload.PackedData[11]);
	return Throughput;
}

void SetHeterogeneousVolumeCallableShaderMajorant(
	float Majorant,
	FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[12] = asuint(Majorant);
}

float GetHeterogeneousVolumeCallableShaderMajorant(
	FPackedPathTracingPayload Payload
)
{
	float Majorant = asfloat(Payload.PackedData[12]);
	return Majorant;
}

// GetDensity

void SetHeterogeneousVolumeCallableShaderTranslatedWorldPos(
	float3 TranslatedWorldPos,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[0] = asuint(TranslatedWorldPos.x);
	Payload.PackedData[1] = asuint(TranslatedWorldPos.y);
	Payload.PackedData[2] = asuint(TranslatedWorldPos.z);
}

float3 GetHeterogeneousVolumeCallableShaderTranslatedWorldPos(
	FPackedPathTracingPayload Payload
)
{
	float3 TranslatedWorldPos = 0;
	TranslatedWorldPos.x = asfloat(Payload.PackedData[0]);
	TranslatedWorldPos.y = asfloat(Payload.PackedData[1]);
	TranslatedWorldPos.z = asfloat(Payload.PackedData[2]);
	return TranslatedWorldPos;
}

void SetHeterogeneousVolumeCallableShaderSigmaT(
	float3 SigmaT,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[0] = asuint(SigmaT.x);
	Payload.PackedData[1] = asuint(SigmaT.y);
	Payload.PackedData[2] = asuint(SigmaT.z);
}

float3 GetHeterogeneousVolumeCallableShaderSigmaT(
	FPackedPathTracingPayload Payload
)
{
	float3 SigmaT = 0;
	SigmaT.x = asfloat(Payload.PackedData[0]);
	SigmaT.y = asfloat(Payload.PackedData[1]);
	SigmaT.z = asfloat(Payload.PackedData[2]);
	return SigmaT;
}

void SetHeterogeneousVolumeCallableShaderAlbedo(
	float3 Albedo,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[3] = asuint(Albedo.x);
	Payload.PackedData[4] = asuint(Albedo.y);
	Payload.PackedData[5] = asuint(Albedo.z);
}

float3 GetHeterogeneousVolumeCallableShaderAlbedo(
	FPackedPathTracingPayload Payload
)
{
	float3 Albedo = 0;
	Albedo.x = asfloat(Payload.PackedData[3]);
	Albedo.y = asfloat(Payload.PackedData[4]);
	Albedo.z = asfloat(Payload.PackedData[5]);
	return Albedo;
}

void SetHeterogeneousVolumeCallableShaderEmission(
	float3 Emission,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[6] = asuint(Emission.x);
	Payload.PackedData[7] = asuint(Emission.y);
	Payload.PackedData[8] = asuint(Emission.z);
}

float3 GetHeterogeneousVolumeCallableShaderEmission(
	FPackedPathTracingPayload Payload
)
{
	float3 Emission = 0;
	Emission.x = asfloat(Payload.PackedData[6]);
	Emission.y = asfloat(Payload.PackedData[7]);
	Emission.z = asfloat(Payload.PackedData[8]);
	return Emission;
}

void SetHeterogeneousVolumeCallableShaderPdf(
	float Pdf,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[0] = asuint(Pdf);
}

float GetHeterogeneousVolumeCallableShaderPdf(
	FPackedPathTracingPayload Payload
)
{
	float Pdf = asfloat(Payload.PackedData[0]);
	return Pdf;
}

void SetHeterogeneousVolumeCallableShaderDistance(
	float Distance,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[1] = asuint(Distance);
}

float GetHeterogeneousVolumeCallableShaderDistance(
	FPackedPathTracingPayload Payload
)
{
	float Distance = asfloat(Payload.PackedData[1]);
	return Distance;
}

void SetHeterogeneousVolumeCallableShaderSigmaBar(
	float3 SigmaBar,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[2] = asuint(SigmaBar.x);
	Payload.PackedData[3] = asuint(SigmaBar.y);
	Payload.PackedData[4] = asuint(SigmaBar.z);
}

float3 GetHeterogeneousVolumeCallableShaderSigmaBar(
	FPackedPathTracingPayload Payload
)
{
	float3 SigmaBar = 0;
	SigmaBar.x = asfloat(Payload.PackedData[2]);
	SigmaBar.y = asfloat(Payload.PackedData[3]);
	SigmaBar.z = asfloat(Payload.PackedData[4]);
	return SigmaBar;
}

void SetHeterogeneousVolumeCallableShaderIsCollision(
	bool bIsCollision,
	inout FPackedPathTracingPayload Payload
)
{
	Payload.PackedData[5] = bIsCollision ? 1 : 0;
}

bool GetHeterogeneousVolumeCallableShaderIsCollision(
	FPackedPathTracingPayload Payload
)
{
	bool bIsCollision = Payload.PackedData[5] == 1 ? true : false;
	return bIsCollision;
}
