// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef USE_PRESHADED_UNIFORM_BUFFER
#define USE_PRESHADED_UNIFORM_BUFFER 0
#endif

#ifndef USE_CALLABLE_SHADERS
#define USE_CALLABLE_SHADERS 0
#endif

#ifndef USE_MULTIPLE_VOLUMES
#define USE_MULTIPLE_VOLUMES 0
#endif 

#if USE_PRESHADED_UNIFORM_BUFFER

int HeterogeneousVolumesNum;
Buffer<int> HeterogeneousVolumesCallableShaderIdIntersect;
Buffer<int> HeterogeneousVolumesCallableShaderIdGetDensityBounds;
Buffer<int> HeterogeneousVolumesCallableShaderIdGetDensity;
Buffer<int> HeterogeneousVolumesCallableShaderIdSampleDistance;
Buffer<int> HeterogeneousVolumesCallableShaderIdGetTransmittance;

void ExpandHitT(float2 RayHitT, inout float2 ResultHitT)
{
	ResultHitT[0] = min(RayHitT[0], ResultHitT[0]);
	ResultHitT[1] = max(RayHitT[1], ResultHitT[1]);
}

FVolumeIntersection HeterogeneousVolumesIntersectPreshading(float3 TranslatedWorldOrigin, float3 WorldDirection, float TMin, float TMax)
{
	float2 RayHitT = float2(TMax, TMin);
#if USE_CALLABLE_SHADERS
	int HeterogeneousVolumesIndex = 0;
#if USE_MULTIPLE_VOLUMES
	for (HeterogeneousVolumesIndex = 0; HeterogeneousVolumesIndex < HeterogeneousVolumesNum; ++HeterogeneousVolumesIndex)
#endif
		if (HeterogeneousVolumesCallableShaderIdIntersect[HeterogeneousVolumesIndex] >= 0)
		{
			FPackedPathTracingPayload Payload = (FPackedPathTracingPayload)0;
			{
				FRayDesc Ray;
				Ray.Origin = TranslatedWorldOrigin;
				Ray.Direction = WorldDirection;
				Ray.TMin = TMin;
				Ray.TMax = TMax;
				SetHeterogeneousVolumeCallableShaderRay(Ray, Payload);

			}

			CallShader(HeterogeneousVolumesCallableShaderIdIntersect[HeterogeneousVolumesIndex], Payload);
			ExpandHitT(GetHeterogeneousVolumeCallableShaderHitT(Payload), RayHitT);
		}
#else
	{
		// Convert ray to local-space
		float3 WorldOrigin = GetHeterogeneousVolumeCallableShaderWorldPos(TranslatedWorldOrigin);
		float3 LocalOrigin = mul(float4(WorldOrigin, 1), PreshadingUniformBuffer.WorldToLocal).xyz;
		float3 LocalDirection = mul(float4(WorldOrigin + WorldDirection, 0), PreshadingUniformBuffer.WorldToLocal).xyz;

		RayHitT = IntersectAABB(
			LocalOrigin,
			LocalDirection,
			TMin,
			TMax,
			PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent,
			PreshadingUniformBuffer.LocalBoundsOrigin + PreshadingUniformBuffer.LocalBoundsExtent
		);
	}
#endif // USE_CALLABLE_SHADERS

	if (RayHitT.y > RayHitT.x)
	{
		return CreateVolumeIntersection(RayHitT.x, RayHitT.y);
	}
	return CreateEmptyVolumeIntersection();
}

void SampleVolumeRenderingCoefficients(
	float3 TranslatedWorldPos,
	inout float3 SigmaT,
	inout float3 Albedo,
	inout float3 Emission
)
{
	float3 LocalPos = mul(float4(GetHeterogeneousVolumeCallableShaderWorldPos(TranslatedWorldPos), 1), PreshadingUniformBuffer.WorldToLocal).xyz;
	float3 LocalBoundsMin = PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent;
	float3 LocalBoundsMax = PreshadingUniformBuffer.LocalBoundsOrigin + PreshadingUniformBuffer.LocalBoundsExtent;
	float3 UVW = (LocalPos - LocalBoundsMin) / (2.0 * PreshadingUniformBuffer.LocalBoundsExtent);

	int MipLevel = 0;
	SigmaT = Texture3DSampleLevel(
		PreshadingUniformBuffer.ExtinctionTexture,
		PreshadingUniformBuffer.TextureSampler,
		UVW,
		MipLevel
	).rgb;

	float3 Majorant = Texture3DSampleLevel(
		PreshadingUniformBuffer.MajorantTexture,
		PreshadingUniformBuffer.PointTextureSampler,
		UVW,
		MipLevel
	).rgb;
	SigmaT = min(SigmaT, Majorant);

	Albedo = Texture3DSampleLevel(
		PreshadingUniformBuffer.AlbedoTexture,
		PreshadingUniformBuffer.TextureSampler,
		UVW,
		MipLevel
	).rgb;

	Emission = Texture3DSampleLevel(
		PreshadingUniformBuffer.EmissionTexture,
		PreshadingUniformBuffer.TextureSampler,
		UVW,
		MipLevel
	).rgb;
}

FVolumeShadedResult HeterogeneousVolumesGetDensityPreshading(float3 TranslatedWorldPos, int HeterogeneousVolumesIndex)
{
	float3 SigmaT = 0;
	float3 Albedo = 0;
	float3 Emission = 0;

#if USE_CALLABLE_SHADERS
	if (HeterogeneousVolumesIndex < 0)
	{
		// Cycle through all volumes, compositing aggregate volumetric properties
		int Index = 0;
#if USE_MULTIPLE_VOLUMES
		for (; Index < HeterogeneousVolumesNum; ++Index)
#endif
		{
			FPackedPathTracingPayload Payload = (FPackedPathTracingPayload)0;
			{
				SetHeterogeneousVolumeCallableShaderTranslatedWorldPos(TranslatedWorldPos, Payload);
			}

			CallShader(HeterogeneousVolumesCallableShaderIdGetDensity[Index], Payload);

			SigmaT += GetHeterogeneousVolumeCallableShaderSigmaT(Payload);
			Albedo += GetHeterogeneousVolumeCallableShaderAlbedo(Payload);
			Emission += GetHeterogeneousVolumeCallableShaderEmission(Payload);
		}
	}
	else if (HeterogeneousVolumesCallableShaderIdGetDensity[HeterogeneousVolumesIndex] >= 0)
	{
		FPackedPathTracingPayload Payload = (FPackedPathTracingPayload)0;
		{
			SetHeterogeneousVolumeCallableShaderTranslatedWorldPos(TranslatedWorldPos, Payload);
		}

		CallShader(HeterogeneousVolumesCallableShaderIdGetDensity[HeterogeneousVolumesIndex], Payload);

		SigmaT = GetHeterogeneousVolumeCallableShaderSigmaT(Payload);
		Albedo = GetHeterogeneousVolumeCallableShaderAlbedo(Payload);
		Emission = GetHeterogeneousVolumeCallableShaderEmission(Payload);
	}
#else
	{
		SampleVolumeRenderingCoefficients(TranslatedWorldPos, SigmaT, Albedo, Emission);
	}
#endif // USE_CALLABLE_SHADERS

	// Return struct
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	Result.SigmaT = SigmaT;
	Result.SigmaSHG = Albedo * SigmaT;
	//Result.SigmaSHG = Albedo;
	Result.PhaseG = GetHeterogeneousVolumeCallableShaderPhaseG();
	Result.Emission = Emission;
	return Result;
}

FVolumeTrackingResult HeterogeneousVolumesMajorantBasedTrackingPreshading(float3 PathThroughput, float3 TranslatedWorldOrigin, float3 WorldDirection, float TMin, float TMax, int bTransmittanceOnly, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence)
{
	FVolumeTrackingResult Result = CreateVolumeTrackingResult(TMin);
	Result.Throughput = PathThroughput;

	// Project ray-span to local space
	float3 TranslatedWorldRayBegin = TranslatedWorldOrigin + WorldDirection * TMin;
	float3 LocalSpanT0 = mul(float4(GetHeterogeneousVolumeCallableShaderWorldPos(TranslatedWorldOrigin + WorldDirection * TMin), 1), PreshadingUniformBuffer.WorldToLocal).xyz;
	float3 LocalSpanT1 = mul(float4(GetHeterogeneousVolumeCallableShaderWorldPos(TranslatedWorldOrigin + WorldDirection * TMax), 1), PreshadingUniformBuffer.WorldToLocal).xyz;

	float3 LocalOrigin = mul(float4(GetHeterogeneousVolumeCallableShaderWorldPos(TranslatedWorldOrigin), 1), PreshadingUniformBuffer.WorldToLocal).xyz;
	float LocalTMin = length(LocalSpanT0 - LocalOrigin);
	float LocalTMax = length(LocalSpanT1 - LocalOrigin);
	float LocalToWorldScale = (TMax - TMin) / (LocalTMax - LocalTMin);

	// Convert to Majorant voxel-space units
	float3 LocalBoundsMin = PreshadingUniformBuffer.LocalBoundsOrigin - PreshadingUniformBuffer.LocalBoundsExtent;
	float3 LocalBoundsDiagonal = 2.0 * PreshadingUniformBuffer.LocalBoundsExtent;
	float3 MajorantVoxelSpanT0 = (LocalSpanT0 - LocalBoundsMin) / LocalBoundsDiagonal * PreshadingUniformBuffer.MajorantResolution;
	float3 MajorantVoxelSpanT1 = (LocalSpanT1 - LocalBoundsMin) / LocalBoundsDiagonal * PreshadingUniformBuffer.MajorantResolution;

	// Determine Majorant voxel-space ray parameters
	float3 MajorantVoxelOrigin = MajorantVoxelSpanT0;
	float3 MajorantVoxelDirection = MajorantVoxelSpanT1 - MajorantVoxelSpanT0;
	float MajorantVoxelTMin = 0.0;
	float MajorantVoxelTMax = length(MajorantVoxelDirection);
	MajorantVoxelDirection /= MajorantVoxelTMax;
	float3 MajorantVoxelDirectionInv = 1.0 / MajorantVoxelDirection;
	float VoxelToWorldScale = (TMax - TMin) / MajorantVoxelTMax;
	float VoxelToLocalScale = (LocalTMax - LocalTMin) / MajorantVoxelTMax;

	if (MajorantVoxelTMin >= MajorantVoxelTMax && !bTransmittanceOnly)
	{
		float LocalToWorldScale = 1.0;
		FVolumeTrackingResult Sample = HeterogeneousVolumesDeltaTracking(1, 0, TMax - TMin, LocalToWorldScale, OverlappingMajorant, RandSequence);
		if (Sample.Distance > 0.0)
		{
			Result = ConcatenateVolumeTrackingResult(Result, Sample);
		}
		return Result;
	}

	float3 MajorantVoxelPos = MajorantVoxelOrigin;
	float3 MajorantVoxelBounds;
	MajorantVoxelBounds.x = sign(MajorantVoxelDirection.x) > 0 ? floor(MajorantVoxelPos.x) + 1 : ceil(MajorantVoxelPos.x) - 1;
	MajorantVoxelBounds.y = sign(MajorantVoxelDirection.y) > 0 ? floor(MajorantVoxelPos.y) + 1 : ceil(MajorantVoxelPos.y) - 1;
	MajorantVoxelBounds.z = sign(MajorantVoxelDirection.z) > 0 ? floor(MajorantVoxelPos.z) + 1 : ceil(MajorantVoxelPos.z) - 1;

	float3 MajorantVoxelBoundsHitT = (MajorantVoxelBounds - MajorantVoxelPos) * MajorantVoxelDirectionInv;
	// Remove floating-point rounding error
	float Epsilon = 1.0e-4;
	if (MajorantVoxelBoundsHitT.x <= Epsilon) MajorantVoxelBoundsHitT.x += abs(MajorantVoxelDirectionInv.x);
	if (MajorantVoxelBoundsHitT.y <= Epsilon) MajorantVoxelBoundsHitT.y += abs(MajorantVoxelDirectionInv.y);
	if (MajorantVoxelBoundsHitT.z <= Epsilon) MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);

	// Ray-march in majorant voxel space
	float MajorantVoxelT = 0;
	while ((MajorantVoxelT < MajorantVoxelTMax) && any(Result.Throughput > Epsilon) && !Result.bIsCollision)
	{
		float MajorantVoxelDeltaT = 0.0;
		if (MajorantVoxelBoundsHitT.x < MajorantVoxelBoundsHitT.y)
		{
			if (MajorantVoxelBoundsHitT.x < MajorantVoxelBoundsHitT.z)
			{
				MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.x - MajorantVoxelT;
				MajorantVoxelBoundsHitT.x += abs(MajorantVoxelDirectionInv.x);
			}
			else
			{
				MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.z - MajorantVoxelT;
				MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);
			}
		}
		else
		{
			if (MajorantVoxelBoundsHitT.y < MajorantVoxelBoundsHitT.z)
			{
				MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.y - MajorantVoxelT;
				MajorantVoxelBoundsHitT.y += abs(MajorantVoxelDirectionInv.y);
			}
			else
			{
				MajorantVoxelDeltaT = MajorantVoxelBoundsHitT.z - MajorantVoxelT;
				MajorantVoxelBoundsHitT.z += abs(MajorantVoxelDirectionInv.z);
			}
		}

		// Clip voxel delta-t by overall voxel ray-length.
		if (MajorantVoxelT + MajorantVoxelDeltaT > MajorantVoxelTMax)
		{
			MajorantVoxelDeltaT = MajorantVoxelTMax - MajorantVoxelT;
		}

		// Convert midpoint to UVW for sampling
		float3 MajorantVoxelEndPos = MajorantVoxelPos + MajorantVoxelDirection * MajorantVoxelDeltaT;
		float3 MajorantVoxelSamplePos = (MajorantVoxelPos + MajorantVoxelEndPos) * 0.5;
		float3 UVW = MajorantVoxelSamplePos / PreshadingUniformBuffer.MajorantResolution;

		int MipLevel = 0;
		float3 Majorant = Texture3DSampleLevel(
			PreshadingUniformBuffer.MajorantTexture,
			// ORIGINAL BUG..
			//PreshadingUniformBuffer.TextureSampler,
			// WITH FIX..
			PreshadingUniformBuffer.PointTextureSampler,
			UVW,
			MipLevel
		).rgb;

		FMajorantData MajorantData;
		MajorantData.Majorant = max(Majorant.x, max(Majorant.y, Majorant.z));
		MergeMajorantData(MajorantData, OverlappingMajorant);

		float3 TranslatedWorldTrackingOrigin = TranslatedWorldRayBegin + WorldDirection * MajorantVoxelT * VoxelToWorldScale;
		float WorldDeltaT = MajorantVoxelDeltaT * VoxelToWorldScale;
		float LocalDeltaT = MajorantVoxelDeltaT * VoxelToLocalScale;

		FVolumeTrackingResult Sample = CreateVolumeTrackingResult(0.0);
		//if (any(MajorantData.Majorant > 0))
		{
			if (bTransmittanceOnly)
			{
				Sample = HeterogeneousVolumesRatioTracking(Result.Throughput, TranslatedWorldTrackingOrigin, WorldDirection, LocalDeltaT, LocalToWorldScale, MajorantData, RandSequence);
			}
			else
			{
				Sample = HeterogeneousVolumesDeltaTracking(1.0, 0.0, LocalDeltaT, LocalToWorldScale, MajorantData, RandSequence);
			}

			if (Sample.Distance < 0)
			{
				return Sample;
			}
		}
		Result = ConcatenateVolumeTrackingResult(Result, Sample);

		MajorantVoxelT += MajorantVoxelDeltaT;
		MajorantVoxelPos = MajorantVoxelEndPos;
	}

	return Result;
}

FVolumeTrackingResult HeterogeneousVolumesSampleDistancePreshading(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, FMajorantData OverlappingMajorant, inout FRandomSequence RandSequence)
{
	bool bTransmittanceOnly = false;
	float MinDistance = TMax;

	FVolumeTrackingResult Result = (FVolumeTrackingResult)0;
	FRandomSequence RandSequenceResult = RandSequence;
#if USE_CALLABLE_SHADERS

	Result.Throughput = PathThroughput;
	Result.Pdf = 1.0;
	Result.Distance = TMax;
	int HeterogeneousVolumesIndex = 0;
#if USE_MULTIPLE_VOLUMES
	for (HeterogeneousVolumesIndex = 0; HeterogeneousVolumesIndex < HeterogeneousVolumesNum; ++HeterogeneousVolumesIndex)
#endif
		if (HeterogeneousVolumesCallableShaderIdSampleDistance[HeterogeneousVolumesIndex])
		{
			FPackedPathTracingPayload Payload = (FPackedPathTracingPayload)0;
			{
				FRayDesc Ray;
				Ray.Origin = Origin;
				Ray.Direction = Direction;
				Ray.TMin = TMin;
				Ray.TMax = TMax;

				SetHeterogeneousVolumeCallableShaderRay(Ray, Payload);
				SetHeterogeneousVolumeCallableShaderThroughput(PathThroughput, Payload);
				SetHeterogeneousVolumeCallableShaderRandomSequence(RandSequence, Payload);
				SetHeterogeneousVolumeCallableShaderMajorant(OverlappingMajorant.Majorant, Payload);
			}

			CallShader(HeterogeneousVolumesCallableShaderIdSampleDistance[HeterogeneousVolumesIndex], Payload);
			RandSequenceResult = GetHeterogeneousVolumeCallableShaderRandomSequence(Payload);

			float Distance = GetHeterogeneousVolumeCallableShaderDistance(Payload);
			bool bIsCollision = GetHeterogeneousVolumeCallableShaderIsCollision(Payload);
			if (bIsCollision && Distance <= MinDistance)
			{
				MinDistance = Distance;

				Result.Throughput = GetHeterogeneousVolumeCallableShaderThroughput(Payload);
				Result.Pdf = GetHeterogeneousVolumeCallableShaderPdf(Payload);
				Result.Distance = GetHeterogeneousVolumeCallableShaderDistance(Payload);
				Result.SigmaBar = GetHeterogeneousVolumeCallableShaderSigmaBar(Payload);
				Result.bIsCollision = GetHeterogeneousVolumeCallableShaderIsCollision(Payload);
				Result.HeterogeneousVolumesIndex = HeterogeneousVolumesIndex;
			}
		}
#else
	{
		Result = HeterogeneousVolumesMajorantBasedTrackingPreshading(PathThroughput, Origin, Direction, TMin, TMax, bTransmittanceOnly, OverlappingMajorant, RandSequence);
	}
#endif // USE_CALLABLE_SHADERS

	RandSequence = RandSequenceResult;
	return Result;
}

float3 HeterogeneousVolumesGetTransmittancePreshading(float3 PathThroughput, float3 Origin, float3 Direction, float TMin, float TMax, inout FRandomSequence RandSequence)
{
	bool bTransmittanceOnly = true;
	FMajorantData EmptyOverlappingMajorant = CreateMajorantData(0.0f);
	float3 Throughput = 1.0;

#if USE_CALLABLE_SHADERS
	int HeterogeneousVolumesIndex = 0;
#if USE_MULTIPLE_VOLUMES
	for (; HeterogeneousVolumesIndex < HeterogeneousVolumesNum; ++HeterogeneousVolumesIndex)
#endif
		if (HeterogeneousVolumesCallableShaderIdGetTransmittance[HeterogeneousVolumesIndex])
		{
			FPackedPathTracingPayload Payload = (FPackedPathTracingPayload)0;
			{
				FRayDesc Ray;
				Ray.Origin = Origin;
				Ray.Direction = Direction;
				Ray.TMin = TMin;
				Ray.TMax = TMax;

				SetHeterogeneousVolumeCallableShaderRay(Ray, Payload);
				SetHeterogeneousVolumeCallableShaderThroughput(PathThroughput, Payload);
				SetHeterogeneousVolumeCallableShaderRandomSequence(RandSequence, Payload);
			}

			CallShader(HeterogeneousVolumesCallableShaderIdGetTransmittance[HeterogeneousVolumesIndex], Payload);

			Throughput *= GetHeterogeneousVolumeCallableShaderThroughput(Payload);
			RandSequence = GetHeterogeneousVolumeCallableShaderRandomSequence(Payload);
		}
#else
	{
		Throughput = HeterogeneousVolumesMajorantBasedTrackingPreshading(PathThroughput, Origin, Direction, TMin, TMax, bTransmittanceOnly, EmptyOverlappingMajorant, RandSequence).Throughput;
	}
#endif // USE_CALLABLE_SHADERS

	return Throughput;
}

#endif // USE_PRESHADED_UNIFORM_BUFFER