// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

FVolumeTrackingResult CreateVolumeTrackingResult(float Distance)
{
	FVolumeTrackingResult Result = (FVolumeTrackingResult)0;
	Result.Throughput = 1;
	Result.Pdf = 0;
	Result.Distance = Distance;
	Result.SigmaBar = 0;
	Result.bIsCollision = false;
	Result.HeterogeneousVolumesIndex = 0;

	return Result;
}

FVolumeTrackingResult ConcatenateVolumeTrackingResult(FVolumeTrackingResult lhs, FVolumeTrackingResult rhs)
{
	FVolumeTrackingResult Result;
	// Result for combined segment
	Result.Throughput = lhs.Throughput * rhs.Throughput;
	Result.Pdf = lhs.Pdf * rhs.Pdf;
	Result.Distance = lhs.Distance + rhs.Distance;
	// Reporting for newest segment
	Result.SigmaBar = rhs.SigmaBar;
	Result.bIsCollision = rhs.bIsCollision;
	Result.HeterogeneousVolumesIndex = rhs.HeterogeneousVolumesIndex;

	return Result;
}

FVolumeTrackingResult HeterogeneousVolumesDeltaTracking(float3 PathThroughput, float LocalTMin, float LocalTMax, float LocalToWorldScale, FMajorantData MajorantData, inout FRandomSequence RandSequence)
{
	FVolumeTrackingResult Sample = CreateVolumeTrackingResult(LocalTMin);
	Sample.SigmaBar = MajorantData.Majorant;

	float RandValue = RandSequence.Get1D();
#define SAMPLE_ACHROMATIC 0
#if SAMPLE_ACHROMATIC
	float DeltaT = -log(1.0 - RandValue) / Sample.SigmaBar.x;
#else
	float DeltaT = SampleSpectralTransmittance(RandValue, Sample.SigmaBar, PathThroughput);
	if (DeltaT < 0)
	{
		return Sample;
	}
#endif

	Sample.Distance = min(LocalTMin + DeltaT, LocalTMax);
	Sample.Throughput = PathThroughput;
	Sample.bIsCollision = Sample.Distance < LocalTMax;
	if (Sample.bIsCollision)
	{
#if SAMPLE_ACHROMATIC
		Sample.Throughput /= Sample.SigmaBar.x;
		float Transmittance = exp(-Sample.SigmaBar.x * DeltaT);
		Sample.Pdf = Transmittance * Sample.SigmaBar.x;
#else
		float4 Evaluation = EvaluateSpectralTransmittanceHit(DeltaT, Sample.SigmaBar, Sample.Throughput);
		Sample.Throughput *= Evaluation.xyz;
		Sample.Pdf = Evaluation.w;
	}
	else
	{
		float4 Evaluation = EvaluateSpectralTransmittanceMiss(LocalTMax - LocalTMin, Sample.SigmaBar, Sample.Throughput);
		Sample.Throughput *= Evaluation.xyz;
		Sample.Pdf = Evaluation.w;
#endif
	}

	Sample.Distance *= LocalToWorldScale;
	return Sample;
}

FVolumeTrackingResult HeterogeneousVolumesRatioTracking(float3 PathThroughput, float3 TranslatedWorldOrigin, float3 WorldDirection, float LocalTMax, float LocalToWorldScale, FMajorantData MajorantData, inout FRandomSequence RandSequence)
{
	FVolumeTrackingResult Sample = CreateVolumeTrackingResult(0);
	Sample.SigmaBar = MajorantData.Majorant;

	while ((Sample.Distance < LocalTMax))
	{
		float RandValue = RandSequence.Get1D();
#if SAMPLE_ACHROMATIC
		float DeltaT = -log(1.0 - RandValue) / Sample.SigmaBar.x;
#else
		float3 CombinedThroughput = PathThroughput * Sample.Throughput;
		float DeltaT = SampleSpectralTransmittance(RandValue, Sample.SigmaBar, CombinedThroughput);
		if (DeltaT < 0)
		{
			Sample.Throughput = 0;
			Sample.Distance = -1;
			return Sample;
		}
#endif

		float LocalCollisionDistance = Sample.Distance + DeltaT;
		if (LocalCollisionDistance < LocalTMax)
		{
			float3 TranslatedWorldSamplePos = TranslatedWorldOrigin + WorldDirection * LocalCollisionDistance * LocalToWorldScale;
			int HeterogeneousVolumesIndex = 0;
			float3 SigmaT = HeterogeneousVolumesGetDensity(TranslatedWorldSamplePos, HeterogeneousVolumesIndex).SigmaT;
			SigmaT = min(SigmaT, Sample.SigmaBar);

			float3 SpectralNullCollisionWeight = max(Sample.SigmaBar - SigmaT, 0);
#if SAMPLE_ACHROMATIC
			Sample.Throughput *= SpectralNullCollisionWeight / Sample.SigmaBar.x;
			float Transmittance = exp(-Sample.SigmaBar.x * DeltaT);
			Sample.Pdf *= Transmittance * Sample.SigmaBar.x;
#else
			float4 EvalResult = EvaluateSpectralTransmittanceHit(DeltaT, Sample.SigmaBar, CombinedThroughput);
			Sample.Throughput *= SpectralNullCollisionWeight * EvalResult.xyz;
			Sample.Pdf *= EvalResult.w;
		}
		else
		{
			float4 EvalResult = EvaluateSpectralTransmittanceMiss(LocalTMax - Sample.Distance, Sample.SigmaBar, CombinedThroughput);
			Sample.Throughput *= EvalResult.xyz;
			Sample.Pdf *= EvalResult.w;
#endif
		}

		Sample.Distance = min(LocalCollisionDistance, LocalTMax);
	}

	Sample.Distance *= LocalToWorldScale;
	return Sample;
}