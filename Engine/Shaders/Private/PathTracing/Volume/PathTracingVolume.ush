// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingVolumeCommon.ush"
#include "PathTracingAtmosphere.ush"
#include "PathTracingClouds.ush"
#include "PathTracingFog.ush"
#include "PathTracingHeterogeneousVolumes.ush"
#include "PathTracingVolumeSampling.ush"

uint VolumeFlags;
int MaxRaymarchSteps;


// Given an input ray, figure out where the provided volume intersects it
FVolumeIntersectionList VolumeIntersect(float3 Origin, float3 Direction, float TMin, float TMax, float PathRoughness, uint VolumeMask)
{
	// NOTE: only this call needs to be guarded by enable flags at the moment since the others can't be reached if this returns nothing
	FVolumeIntersectionList Result = CreateEmptyVolumeIntersectionList();
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
	{
		Result.Add(VOLUMEID_ATMOSPHERE, AtmosphereIntersect(Origin, Direction, TMin, TMax));
	}
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_CLOUDS)
	{
		Result.Add(VOLUMEID_CLOUDS, CloudsIntersect(Origin, Direction, TMin, TMax, PathRoughness));
	}
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_FOG)
	{
		Result.Add(VOLUMEID_FOG, FogIntersect(Origin, Direction, TMin, TMax));
	}
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		Result.Add(VOLUMEID_HETEROGENEOUS_VOLUMES, HeterogeneousVolumesIntersect(Origin, Direction, TMin, TMax));
	}
	return Result;
}

FPackedPathTracingPayload VolumeGetBlockerHit(float3 Origin, float3 Direction, float HitT, bool bIsCameraRay)
{
	// NOTE: only atmosphere supports blockers, so if this gets called it is safe to assume its for the atmosphere volume
	// If we ever add other volume types that imply blockers, we would have to add the ID of the volume type as input
	return AtmosphereGetBlockerHit(Origin, Direction, HitT, bIsCameraRay);
}

// Given an input ray, figure out the bounds on density. The T interval may be smaller than the one returned by VolumeIntersect
FVolumeDensityBounds VolumeGetDensityBounds(float3 Origin, float3 Direction, FVolumeIntersectionInterval Interval)
{
	// CloudDensity will only be non-zero if the Interval overlaps with Clouds
	FVolumeDensityBounds Result = CreateVolumeDensityBound(0, Interval.CloudDensity);
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
	{
		MergeVolumeDensityBounds(Result, AtmosphereGetDensityBounds(Origin, Direction, Interval.VolumeTMin, Interval.VolumeTMax));
	}
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_FOG)
	{
		MergeVolumeDensityBounds(Result, FogGetDensityBounds(Origin, Direction, Interval.VolumeTMin, Interval.VolumeTMax));
	}
#if 0
	// NOTE: this case is not possible because we don't use density bounds directly when heterogeneous volumes are present
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		MergeVolumeDensityBounds(Result, HeterogeneousVolumesGetDensityBounds(Origin, Direction, Interval.VolumeTMin, Interval.VolumeTMax));
	}
#endif
	return Result;
}

// Given a point in world space, return the amount of volume and its scattering properties
FVolumeShadedResult VolumeGetDensity(float3 TranslatedWorldPos, int HeterogeneousVolumesIndex, uint VolumeMask, bool bIsCameraRay)
{
	FVolumeShadedResult Result = (FVolumeShadedResult)0;
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
	{
		MergeVolumeShadedResult(Result, AtmosphereGetDensity(TranslatedWorldPos), bIsCameraRay && (VolumeFlags & PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE) != 0);
	}
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_FOG)
	{
		MergeVolumeShadedResult(Result, FogGetDensity(TranslatedWorldPos), bIsCameraRay && (VolumeFlags & PATH_TRACER_VOLUME_HOLDOUT_FOG) != 0);
	}
	if (VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		MergeVolumeShadedResult(Result, HeterogeneousVolumesGetDensity(TranslatedWorldPos, HeterogeneousVolumesIndex), bIsCameraRay && (VolumeFlags & PATH_TRACER_VOLUME_HOLDOUT_HETEROGENEOUS_VOLUMES) != 0);
	}
	return Result;
}


#define PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP_BODY(GetDensityFunc)								\
		/* take stochastic steps along the ray to estimate transmittance (null scattering) */		\
		float3 ColorChannelPdf = Throughput;														\
		/* Sample the distance to the next interaction */											\
		float RandValue = RandSequence.Get1D();														\
		float DeltaT = SampleSpectralTransmittance(RandValue.x, SigmaBar, ColorChannelPdf);			\
		if (DeltaT < 0.0)																			\
		{																							\
			/* no more energy left in the path */													\
			break;																					\
		}																							\
		if (TMin + DeltaT < TMax)																	\
		{																							\
			TMin += DeltaT;																			\
			/* our ray marching step stayed inside the atmo and is still in front of the next hit */\
			/* Compute transmittance through the bounding homogeneous medium (both real and fictitious particles) */	\
			Throughput *= EvaluateSpectralTransmittanceHit(DeltaT, SigmaBar, ColorChannelPdf).xyz;	\
			float3 WorldPos = Origin + TMin * Direction;											\
			/* clamp to make sure we never exceed the majorant (should not be the case, but need to avoid any possible numerical issues) */	\
			float3 SigmaT = min(GetDensityFunc(WorldPos).SigmaT, SigmaBar);							\
			float3 SigmaN = SigmaBar - SigmaT;														\
			/* keep tracing through the volume */													\
			Throughput *= SigmaN;																	\
			Throughput *= LowThroughputClampFactor(Throughput); 									\
			if (!any(Throughput > 0))																\
			{																						\
				break;																				\
			}																						\
		}																							\
		else                                                                                        \
		{																							\
			/* update the path throughput, knowing that we escaped the medium*/						\
			Throughput *= EvaluateSpectralTransmittanceMiss(TMax - TMin, SigmaBar, ColorChannelPdf).xyz;	\
			/* exit the ray marching loop */														\
			break;																					\
		}


// Reference implementation of transmittance that uses only GetDensity and GetDensityBounds
// express the inner loop here as a macro so we can stamp down a separate copy per volume ID and avoid a nested loop during ray marching
#define PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP(GetDensityBoundsFunc, GetDensityFunc)				\
	/* Limit number of steps to prevent timeouts // FIXME: This biases the result! */				\
	for (int Step = 0; Step < MaxRaymarchSteps; Step++)												\
	{																								\
		float3 SigmaBar = GetDensityBoundsFunc(Origin, Direction, TMin, TMax).SigmaMax;				\
		PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP_BODY(GetDensityFunc);								\
	}

// Same as above, but when SigmaBar is constant for the whole loop (and defined outside)
#define PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP_CONSTANT_BOUNDS(GetDensityFunc)					\
	/* Limit number of steps to prevent timeouts // FIXME: This biases the result! */				\
	for (int Step = 0; Step < MaxRaymarchSteps; Step++)												\
	{																								\
		PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP_BODY(GetDensityFunc);								\
	}


// The returned value factors in the initial throughput. This is used to avoid sampling too much if throughput reaches 0.
// Return the transmittance along a ray segment
float3 VolumeGetTransmittance(float3 StartThroughput, float3 Origin, float3 Direction, FVolumeIntersectionInterval Interval, inout FRandomSequence RandSequence)
{
	float3 Throughput = StartThroughput;
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
	{
		float TMin = Interval.VolumeTMin;
		float TMax = Interval.VolumeTMax;
		if (VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE)
		{
			Throughput *= AtmosphereGetTransmittance(Origin, Direction, TMin, TMax);
		}
		else
		{
			PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP(AtmosphereGetDensityBounds, AtmosphereGetDensity);
		}
	}
	// NOTE: This function does not include cloud transmittance as it should only be called when cloud density
	//       has already been accounted for or is not present on this interval.
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_FOG)
	{
		float TMin = Interval.VolumeTMin;
		float TMax = Interval.VolumeTMax;
		if (VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE)
		{
			Throughput *= FogGetTransmittance(Origin, Direction, TMin, TMax);
		}
		else
		{
			PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP(FogGetDensityBounds, FogGetDensity);
		}
	}
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		float TMin = Interval.VolumeTMin;
		float TMax = Interval.VolumeTMax;
		Throughput = HeterogeneousVolumesGetTransmittance(Throughput, Origin, Direction, TMin, TMax, RandSequence);
	}
	return Throughput;
}

// Return the transmittance along a whole intersection list
// Each volume type has its own range unlike the function above. This is used for shadow rays, so this version adds the CloudFactor for cloud shadows
float3 VolumeGetTransmittance(float3 StartThroughput, float3 Origin, float3 Direction, FVolumeIntersectionList IntersectionList, inout FRandomSequence RandSequence, float CloudFactor)
{
	float3 Throughput = StartThroughput;
	if (IntersectionList.VolumeMask & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE)
	{
		float TMin = IntersectionList.VolumeTMin[VOLUMEID_ATMOSPHERE];
		float TMax = IntersectionList.VolumeTMax[VOLUMEID_ATMOSPHERE];
		if (VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE)
		{
			Throughput *= AtmosphereGetTransmittance(Origin, Direction, TMin, TMax);
		}
		else
		{
			PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP(AtmosphereGetDensityBounds, AtmosphereGetDensity);
		}
	}
#if PATH_TRACER_USE_CLOUD_SHADER
	if (IntersectionList.VolumeMask & PATH_TRACER_VOLUME_ENABLE_CLOUDS)
	{
		float TMin = IntersectionList.VolumeTMin[VOLUMEID_CLOUDS];
		float TMax = IntersectionList.VolumeTMax[VOLUMEID_CLOUDS];
		float SigmaBar = IntersectionList.CloudDensity;
		FPackedPathTracingPayload CloudPayload = (FPackedPathTracingPayload) 0;
		CloudPayload.SetVolumetricCallableShaderInput(Origin, Direction, TMin, TMax, RandSequence, PATH_TRACER_VOLUME_CALLABLE_FLAGS_TRANSMITTANCE, Throughput, SigmaBar);
		CloudPayload.SetVolumetricCallableShaderInputCloudFactor(CloudFactor);
		CallShader(CloudCallableShaderId, CloudPayload);
		RandSequence = CloudPayload.GetVolumetricCallableShaderOutputRandSeq();
		Throughput = CloudPayload.GetVolumetricCallableShaderOutputTransmittanceThroughput();
	}
#endif
	if (IntersectionList.VolumeMask & PATH_TRACER_VOLUME_ENABLE_FOG)
	{
		float TMin = IntersectionList.VolumeTMin[VOLUMEID_FOG];
		float TMax = IntersectionList.VolumeTMax[VOLUMEID_FOG];
		if (VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE)
		{
			Throughput *= FogGetTransmittance(Origin, Direction, TMin, TMax);
		}
		else
		{
			PATH_TRACER_REFERENCE_TRANSMITTANCE_LOOP(FogGetDensityBounds, FogGetDensity);
		}
	}
	if (IntersectionList.VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		float TMin = IntersectionList.VolumeTMin[VOLUMEID_HETEROGENEOUS_VOLUMES];
		float TMax = IntersectionList.VolumeTMax[VOLUMEID_HETEROGENEOUS_VOLUMES];
		Throughput = HeterogeneousVolumesGetTransmittance(Throughput, Origin, Direction, TMin, TMax, RandSequence);
	}
	return Throughput;
}

FVolumeShadedResult VolumeGetDensityAndTransmittance(float3 Origin, float3 Direction, FVolumeIntersectionInterval Interval, uint Bounce, inout FRandomSequence RandSequence, inout float3 Throughput, inout float CloudFactor)
{
	FVolumeShadedResult Result = (FVolumeShadedResult) 0;
#if PATH_TRACER_USE_CLOUD_SHADER
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_CLOUDS)
	{
		float TMin = Interval.VolumeTMin;
		float TMax = Interval.VolumeTMax;
		float SigmaBar = Interval.CloudDensity;
		FPackedPathTracingPayload CloudPayload = (FPackedPathTracingPayload) 0;
		uint Flags = Interval.VolumeMask;
		Flags |= VolumeFlags & ~PATH_TRACER_VOLUME_ENABLE_MASK;
		Flags |= PATH_TRACER_VOLUME_CALLABLE_FLAGS_TRANSMITTANCE;
		Flags |= PATH_TRACER_VOLUME_CALLABLE_FLAGS_GET_SAMPLE;
		Flags |= min(PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_MASK, Bounce << PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_SHIFT);
		CloudPayload.SetVolumetricCallableShaderInput(Origin, Direction, TMin, TMax, RandSequence, Flags, Throughput, SigmaBar);
		CloudPayload.SetVolumetricCallableShaderInputCloudFactor(1.0);
		CallShader(CloudCallableShaderId, CloudPayload);
		RandSequence = CloudPayload.GetVolumetricCallableShaderOutputRandSeq();
		Throughput = CloudPayload.GetVolumetricCallableShaderOutputTransmittanceThroughput();

		Result = (FVolumeShadedResult) 0;
		// NOTE: SigmaT is not used, SigmaS values are weights
		Result.SigmaSRayleigh = CloudPayload.GetVolumetricCallableShaderOutputRayleighWeight();
		float4 HGData = CloudPayload.GetVolumetricCallableShaderOutputHG();
		Result.SigmaSHG = HGData.xyz;
		Result.PhaseG = HGData.w;
		Result.SigmaSDualHG = CloudPayload.GetVolumetricCallableShaderOutputDualHGWeight();
		Result.DualPhaseData = CloudPayload.GetVolumetricCallableShaderOutputDualHGPhaseData();
		CloudFactor = CloudPayload.GetVolumetricCallableShaderOutputCloudDensityFactor();
	}
	else
#endif
	{
		// non-cloud case, just get density of all volumes
		int HeterogeneousVolumesIndex = -1;
		Result = VolumeGetDensity(Origin + Direction * Interval.VolumeTMax, HeterogeneousVolumesIndex, Interval.VolumeMask, Bounce == 0);
		CloudFactor = 1.0;
		// Factor in SigmaS to throughput
		Throughput *= Result.SigmaSRayleigh + Result.SigmaSHG + Result.SigmaSDualHG;
	}
	// Factor in remainder of transmittance terms from non-cloud volumes along the segment if we still have some energy left
	if (any(Throughput > 0))
	{
		Throughput = VolumeGetTransmittance(Throughput, Origin, Direction, Interval, RandSequence);
	}
	return Result;
}

FVolumeTrackingResult VolumeSampleDistance(float3 PathThroughput, float3 Origin, float3 Direction, FVolumeIntersectionInterval Interval, bool bIsCameraRay, inout FRandomSequence RandSequence)
{
	FVolumeDensityBounds VolumeDensityBounds = VolumeGetDensityBounds(Origin, Direction, Interval);
	float3 SigmaBar = VolumeDensityBounds.SigmaMax;
	if ((VolumeFlags & PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE) == 0)
	{
		// if we are not using analytical transmittance, a tight majorant could prevent us from "seeing" hits that match the density exactly
		// leading to heavy noise on bright objects embedded in the volume
		// however if we can track transmittance analytically, this workaround is not needed as we will re-compute a precise answer after ray marching
		SigmaBar *= bIsCameraRay ? 2 : 1;
	}

	FVolumeTrackingResult TrackingResult;
	if (Interval.VolumeMask & PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES)
	{
		// Interval is crossing a heterogeneous volume
		FMajorantData OverlappingMajorant = CreateMajorantData(SigmaBar);
		TrackingResult = HeterogeneousVolumesSampleDistance(PathThroughput, Origin, Direction, Interval.VolumeTMin, Interval.VolumeTMax, OverlappingMajorant, RandSequence);
	}
	else
	{
		// Interval is only crossing atmo/clouds/fog
		float RandValue = RandSequence.Get1D();
		TrackingResult.Distance = SampleSpectralTransmittance(RandValue, SigmaBar, PathThroughput);
		if (TrackingResult.Distance < 0.0)
		{
			return TrackingResult;
		}

		TrackingResult.SigmaBar = SigmaBar;
		TrackingResult.Throughput = PathThroughput;
		TrackingResult.bIsCollision = Interval.VolumeTMin + TrackingResult.Distance < Interval.VolumeTMax;
		if (TrackingResult.bIsCollision)
		{
			float4 Evaluation = EvaluateSpectralTransmittanceHit(TrackingResult.Distance, SigmaBar, PathThroughput);
			TrackingResult.Throughput *= Evaluation.xyz;
			TrackingResult.Pdf = Evaluation.w;
			TrackingResult.Distance += Interval.VolumeTMin;
		}
		else
		{
			float4 Evaluation = EvaluateSpectralTransmittanceMiss(Interval.VolumeTMax - Interval.VolumeTMin, SigmaBar, PathThroughput);
			TrackingResult.Throughput *= Evaluation.xyz;
			TrackingResult.Pdf = Evaluation.w;
		}
	}

	return TrackingResult;
}
