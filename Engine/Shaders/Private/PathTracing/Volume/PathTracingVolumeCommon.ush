// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FVolumeIntersection
{
	float VolumeTMin;
	float VolumeTMax;
	float BlockerHitT;
	float MaxDensity;

	bool HitVolume()
	{
		return VolumeTMin < VolumeTMax;
	}

	bool HitBlocker()
	{
		return BlockerHitT > 0.0;
	}
};

FVolumeIntersection CreateVolumeIntersection(float TMin, float TMax, float BlockerT = 0.0)
{
	FVolumeIntersection Result = (FVolumeIntersection)0;
	Result.VolumeTMin = TMin;
	Result.VolumeTMax = TMax;
	Result.BlockerHitT = BlockerT;
	return Result;
}

FVolumeIntersection CreateVolumeIntersectionWithDensity(float TMin, float TMax, float MaxDensity)
{
	FVolumeIntersection Result = (FVolumeIntersection)0;
	Result.VolumeTMin = TMin;
	Result.VolumeTMax = TMax;
	Result.MaxDensity = MaxDensity;
	return Result;
}

FVolumeIntersection CreateEmptyVolumeIntersection()
{
	return (FVolumeIntersection)0;
}


// representes an interval along the ray over which the specified volumes are known to exist
struct FVolumeIntersectionInterval
{
	float VolumeTMin;
	float VolumeTMax;
	uint VolumeMask;
	float CloudDensity;
};

struct FVolumeIntersectionList
{
	float4 VolumeTMin;
	float4 VolumeTMax;
	float CloudDensity;
	uint VolumeMask;

	// only need to track one blocker (assume they are opaque)
	float BlockerHitT;

	bool HitVolume()
	{
		return VolumeMask != 0;
	}

	bool HitBlocker()
	{
		return BlockerHitT > 0;
	}

	void Add(uint ID, FVolumeIntersection VolumeIntersection)
	{
		// merge the provided hit into the list
		if (VolumeIntersection.HitVolume())
		{
			VolumeTMin[ID] = VolumeIntersection.VolumeTMin;
			VolumeTMax[ID] = VolumeIntersection.VolumeTMax;
			if (ID == VOLUMEID_CLOUDS)
			{
				CloudDensity = VolumeIntersection.MaxDensity;
			}
			VolumeMask |= PATH_TRACER_VOLUME_ENABLE_BIT << ID;
		}
		if (VolumeIntersection.HitBlocker())
		{
			if (!HitBlocker() || BlockerHitT > VolumeIntersection.BlockerHitT)
			{
				BlockerHitT = VolumeIntersection.BlockerHitT;
			}
		}
	}

	// Return the interval of TValues closest to the front of the list
	FVolumeIntersectionInterval GetCurrentInterval()
	{
		FVolumeIntersectionInterval Result = (FVolumeIntersectionInterval)0;
		for (int Index = 0; Index < PATH_TRACER_MAX_VOLUMES; Index++)
		{
			uint Mask = PATH_TRACER_VOLUME_ENABLE_BIT << Index;
			if ((VolumeMask & Mask) == 0)
			{
				continue; // not valid
			}

			float TMin = VolumeTMin[Index];
			float TMax = VolumeTMax[Index];
			if (Result.VolumeMask == 0)
			{
				// we haven't stored the start interval yet
				Result.VolumeTMin = TMin;
				Result.VolumeTMax = TMax;
				Result.VolumeMask = Mask;
			}
			else
			{
				if (Result.VolumeTMin == TMin)
				{
					// both segments start together, figure out which ends first
					Result.VolumeTMax = min(Result.VolumeTMax, TMax);
				}
				else if (Result.VolumeTMin < TMin)
				{
					// what we have so far _starts_ before the current segment
					// so either the two segments are disjoint (Result.VolumeTMax is closer)
					// or the two overlap, and we need to stop at the next transition line
					Result.VolumeTMax = min(Result.VolumeTMax, TMin);
				}
				else
				{
					// segments could be disjoint (TMax is closer) or overlapping (VolumeTMin is closer)
					Result.VolumeTMax = min(TMax, Result.VolumeTMin);
					Result.VolumeTMin = TMin; // we just established this one is closer
				}
			}
		}
		// now that we established the distance, figure out which IDs overlap with the resolved segment
		Result.VolumeMask = 0;
		for (int Index2 = 0; Index2 < PATH_TRACER_MAX_VOLUMES; Index2++)
		{
			uint Mask = PATH_TRACER_VOLUME_ENABLE_BIT << Index2;
			if ((VolumeMask & Mask) == 0)
			{
				continue; // not overlapping
			}
			if (Result.VolumeTMax > VolumeTMin[Index2] &&
				Result.VolumeTMin < VolumeTMax[Index2])
			{
				Result.VolumeMask |= Mask;
			}
		}
		// Transfer CloudDensity to Interval if this interval includes clouds
		Result.CloudDensity = Result.VolumeMask & PATH_TRACER_VOLUME_ENABLE_CLOUDS ? CloudDensity : 0; 
		return Result;
	}

	// Update all active intervals, knowing we have processed everything up to distance 'T'
	FVolumeIntersectionList Update(float T)
	{
		FVolumeIntersectionList Result = (FVolumeIntersectionList)0;
		// clip the current intervals, knowning that we are "T" away
		for (int Index = 0; Index < PATH_TRACER_MAX_VOLUMES; Index++)
		{
			uint Mask = PATH_TRACER_VOLUME_ENABLE_BIT << Index;
			if ((VolumeMask & Mask) == 0)
			{
				continue; // not valid
			}

			float TMax = VolumeTMax[Index];
			if (T < TMax)
			{
				// still valid? then add it back in
				Result.VolumeTMin[Index] = max(T, VolumeTMin[Index]);
				Result.VolumeTMax[Index] = TMax;
				Result.VolumeMask |= Mask;
			}
		}
		Result.BlockerHitT = BlockerHitT;
		Result.CloudDensity = CloudDensity;
		return Result;
	}
};

FVolumeIntersectionList CreateEmptyVolumeIntersectionList()
{
	return (FVolumeIntersectionList)0;
}


// Represent a strict lower and upper bound on the values that can be returned by VolumeGetDensity along the given ray
struct FVolumeDensityBounds
{
	float3 SigmaMin; // minorant (control extinction)
	float3 SigmaMax; // majorant (upper bound)
};

FVolumeDensityBounds CreateVolumeDensityBound(float3 SigmaMin, float3 SigmaMax)
{
	FVolumeDensityBounds Result = (FVolumeDensityBounds)0;
	Result.SigmaMin = SigmaMin;
	Result.SigmaMax = SigmaMax;
	return Result;
}

void MergeVolumeDensityBounds(inout FVolumeDensityBounds Merged, FVolumeDensityBounds Other)
{
	Merged.SigmaMin += Other.SigmaMin;
	Merged.SigmaMax += Other.SigmaMax;
}


// The result of sampling a volume at a given point
// See PathTracingMedium.usf for the implementation of the scattering model
struct FVolumeShadedResult
{
	float3 SigmaT;         // extinction
	float3 SigmaH;         // holdout density
	float3 SigmaSRayleigh; // Rayleight scattering coefficient
	float3 SigmaSHG;       // Henyey-Greenstein scattering coefficient (for atmosphere, fog and heterogeneous volumes)
	float3 SigmaSDualHG;   // Dual Henyey-Greenstein scattering coefficient (for clouds)
	float3 Emission;
	float PhaseG;          // 'g' term for HG lobe
	float3 DualPhaseData;  // g1,g2,blend for DualHG lobe
};


FPathTracingPayload CreateMediumHitPayload(float HitT, float3 TranslatedWorldPos, FVolumeShadedResult ShadedResult)
{
	FPathTracingPayload Result = (FPathTracingPayload)0; // clear all fields
	Result.HitT = HitT;
	Result.TranslatedWorldPos = TranslatedWorldPos;
	Result.PathTracingShadingModelID = PATH_TRACING_SHADINGMODELID_MEDIUM;
	Result.BSDFOpacity = 1.0;
	Result.PrimitiveLightingChannelMask = 7;
	Result.SetFrontFace();
	const float3 SigmaS = ShadedResult.SigmaSRayleigh + ShadedResult.SigmaSHG + ShadedResult.SigmaSDualHG;
	const float3 RayleighWeight = select(SigmaS > 0.0, ShadedResult.SigmaSRayleigh / SigmaS, 0.0);
	const float3 HGWeight       = select(SigmaS > 0.0, ShadedResult.SigmaSHG       / SigmaS, 0.0);
	const float3 DualHGWeight   = select(SigmaS > 0.0, ShadedResult.SigmaSDualHG   / SigmaS, 0.0);
	Result.SetBaseColor(RayleighWeight);
	Result.SetHG(HGWeight, ShadedResult.PhaseG);
	Result.SetDualHG(DualHGWeight, ShadedResult.DualPhaseData);
	Result.SetCloudFactor(1.0);
	#if SUBSTRATE_PATH_TRACING_GLINTS
	Result.GlintValue = 1; // avoid encode/decode cycle zero'ing out some parameters
	#endif
	return Result;
}

// Merge the shaded result "R" into "M"
void MergeVolumeShadedResult(inout FVolumeShadedResult M, FVolumeShadedResult R, bool bAsHoldout)
{
	if (bAsHoldout)
	{
		M.SigmaT += R.SigmaT;
		M.SigmaH += R.SigmaT;
	}
	else
	{
		M.SigmaT         += R.SigmaT;
		M.SigmaSRayleigh += R.SigmaSRayleigh;
		M.SigmaSHG       += R.SigmaSHG;
		M.SigmaSDualHG   += R.SigmaSDualHG;
		M.Emission       += R.Emission;
		// when combining across volumes, we track dual HG on its own to simplify the logic even though it technicaly create 3 distinct hg lobes
		// A possible improvement we could make in the future is reduce this to only 2 HG lobes with improved blending here.
		float Q1r = R.SigmaSHG.x + R.SigmaSHG.y + R.SigmaSHG.z;
		float Q1m = M.SigmaSHG.x + M.SigmaSHG.y + M.SigmaSHG.z;
		float Q2r = R.SigmaSDualHG.x + R.SigmaSDualHG.y + R.SigmaSDualHG.z;
		float Q2m = M.SigmaSDualHG.x + M.SigmaSDualHG.y + M.SigmaSDualHG.z;
		M.PhaseG        = lerp(M.PhaseG       , R.PhaseG       , Q1m > 0 ? Q1r / Q1m : 0.0);
		M.DualPhaseData = lerp(M.DualPhaseData, R.DualPhaseData, Q2m > 0 ? Q2r / Q2m : 0.0);
	}
}

// represent a stochastically chosen volume element that is guaranteed to lie between 
struct FVolumeSegment
{
	float3 Throughput; // path contribution from the start of the volume segment
	FVolumeIntersectionInterval Interval;

	bool IsValid() { return Interval.VolumeTMin < Interval.VolumeTMax; }
};

FVolumeSegment CreateEmptyVolumeSegment()
{
	return (FVolumeSegment)0;
}
