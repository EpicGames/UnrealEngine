// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// Returns a distance sampled proportionally to transmittance with an extinction of Sigma, using the provided color channel pdf for weighting the probabilities
float SampleSpectralTransmittance(float RandValue, float3 Sigma, float3 ColorChannelPdf)
{
	float3 ColorChannelCdf = float3(
		ColorChannelPdf.x,
		ColorChannelPdf.x + ColorChannelPdf.y,
		ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	if (ColorChannelCdf.z > 0)
	{
		const float OneMinusEpsilon = 0.99999994; // 32-bit float just before 1.0
		float q = RandValue * ColorChannelCdf.z;
		if (q < ColorChannelCdf.x)
		{
			float RescaleRand = min(q / ColorChannelCdf.x, OneMinusEpsilon);
			return -log(1 - RescaleRand) / Sigma.x;
		}
		else if (q < ColorChannelCdf.y)
		{
			float RescaleRand = min((q - ColorChannelCdf.x) / (ColorChannelCdf.y - ColorChannelCdf.x), OneMinusEpsilon);
			return -log(1 - RescaleRand) / Sigma.y;
		}
		else
		{
			float RescaleRand = min((q - ColorChannelCdf.y) / (ColorChannelCdf.z - ColorChannelCdf.y), OneMinusEpsilon);
			return -log(1 - RescaleRand) / Sigma.z;
		}
	}
	// all channels have 0 probability
	return -1.0;
}


float4 EvaluateSpectralTransmittanceHit(float SampledT, float3 Sigma, float3 ColorChannelPdf)
{
	// normalize the pdf (to match code above)
	ColorChannelPdf *= rcp(ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	float3 Transmittance = exp(-SampledT * Sigma);
	float3 TransmittancePdf = Sigma * Transmittance; // probability of reaching the sampled point
	float MisPdf = dot(ColorChannelPdf, TransmittancePdf);
	return MisPdf > 0 ? float4(Transmittance / MisPdf, MisPdf) : 0.0;
}

float4 EvaluateSpectralTransmittanceMiss(float MaxT, float3 Sigma, float3 ColorChannelPdf)
{
	// normalize the pdf (to match code above)
	ColorChannelPdf *= rcp(ColorChannelPdf.x + ColorChannelPdf.y + ColorChannelPdf.z);
	float3 Transmittance = exp(-MaxT * Sigma);
	float3 TransmittancePdf = Transmittance; // probability of going past MaxT (integral of the pdf from MaxT to infinity)
	float MisPdf = dot(ColorChannelPdf, TransmittancePdf);
	return MisPdf > 0 ? float4(Transmittance / MisPdf, MisPdf) : 0.0;
}

// Factor to multiply the path throughput by so that paths that have low overall contribution can get terminated earlier
float LowThroughputClampFactor(float3 Throughput)
{
	// early termination - fades off from 0.002 to 0.001 and then will be 0
	return saturate(1000.0 * max3(Throughput.x, Throughput.y, Throughput.z) - 1.0);
}