// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsFieldBuilder.ush
=============================================================================*/

#pragma once

#include "Common.ush"
#include "PhysicsFieldEval.ush"

/* -----------------------------------------------------------------
 * Field System constants and context
 * -----------------------------------------------------------------
 */

RWBuffer<float> FieldClipmap;

Buffer<int> CellsOffsets;
Buffer<uint4> CellsMin;
Buffer<uint4> CellsMax;

int ClipmapResolution;
float ClipmapDistance;
float3 ClipmapCenter;
int ClipmapCount;
int ClipmapExponent;

int ValidCount;
int ValidTargets[MAX_PHYSICS_FIELD_TARGETS];
int TargetsIndex[MAX_PHYSICS_FIELD_TARGETS];

int NumCells;
int TargetCount;

Buffer<float> NodesParams;
Buffer<int> NodesOffsets;
Buffer<int> TargetsOffsets;
float TimeSeconds;


uint3 GetCellVoxel( in int ThreadId, in int CellIndex)
{
	const int LocalIndex = ThreadId - CellsOffsets[CellIndex];
	const uint4 CellsDelta = CellsMax[CellIndex] - CellsMin[CellIndex];
					
	const int IndexZ = LocalIndex / (CellsDelta.y * CellsDelta.x);
	const int IndexT = LocalIndex % (CellsDelta.y * CellsDelta.x);
	const int IndexY = IndexT / CellsDelta.x;
	const int IndexX = IndexT % CellsDelta.x;
	
	return CellsMin[CellIndex].xyz + uint3(IndexX, IndexY, IndexZ);
}

int GetBufferOffset(in int CellTarget, in int CellClipmap, in uint3 CellVoxel)
{
	const int DatasOffset = CellClipmap + TargetsIndex[CellTarget] * ClipmapCount;
	return (CellVoxel.x + CellVoxel.y * ClipmapResolution + (CellVoxel.z + ClipmapResolution * DatasOffset) * ClipmapResolution * ClipmapResolution);
}	

float GetClipmapFade(in int CellClipmap, in uint3 CellVoxel)
{
	float ClipmapFade = 1.0;
	if (CellClipmap == ClipmapCount - 1)
	{
		const float LimitDistance = (float) ClipmapResolution / 8.0;
		const uint3 BorderDirection = min(uint3(ClipmapResolution - 1, ClipmapResolution - 1, ClipmapResolution - 1) - CellVoxel.xyz, CellVoxel.xyz);
		const float BorderDistance = min(min(BorderDirection.x, BorderDirection.y), BorderDirection.z);
		ClipmapFade = (BorderDistance < LimitDistance) ? (BorderDistance * BorderDistance / (LimitDistance * LimitDistance)) : 1.0;
	}
	return ClipmapFade;
}

float3 GetSamplePosition(in float ClipmapExtent, in uint3 CellVoxel)
{
	const float3 SampleUV = (CellVoxel.xyz) / float3(ClipmapResolution - 1, ClipmapResolution - 1, ClipmapResolution - 1);
	return ClipmapCenter + (SampleUV - 0.5) * 2.0 * ClipmapExtent;	
}

[numthreads(BUILD_FIELD_THREAD_GROUP_SIZE, 1, 1)]
void BuildPhysicsFieldClipmapCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID)
{
	if ((DispatchThreadId.x < NumCells))
	{
		int CellIndex = 0;
		
		int CellTarget = -1;
		for (int ValidIndex = 0; ValidIndex < ValidCount; ++ValidIndex, CellIndex += ClipmapCount)
		{
			if ((DispatchThreadId.x >= CellsOffsets[CellIndex]) && (DispatchThreadId.x < CellsOffsets[CellIndex + ClipmapCount]))
			{
				CellTarget = ValidTargets[ValidIndex];
				break;
			}
		}
		if (CellTarget != -1)
		{
			int CellClipmap = -1;
			for (int ClipmapIndex = 0; ClipmapIndex < ClipmapCount; ++ClipmapIndex, ++CellIndex)
			{
				if ((DispatchThreadId.x >= CellsOffsets[CellIndex]) && (DispatchThreadId.x < CellsOffsets[CellIndex + 1]))
				{
					CellClipmap = ClipmapIndex;
					break;
				}
			}
			if (CellClipmap != -1)
			{
				const uint3 CellVoxel = GetCellVoxel(DispatchThreadId.x, CellIndex);
		
				if (all(CellVoxel.xyz < int3(ClipmapResolution, ClipmapResolution, ClipmapResolution)))
				{
					const float ClipmapFade = GetClipmapFade(CellClipmap,CellVoxel);
					const float ClipmapExtent = ClipmapDistance * pow(ClipmapExponent, CellClipmap + 1 - ClipmapCount);
		
					const float3 SamplePosition = GetSamplePosition(ClipmapExtent, CellVoxel);
					const int BufferOffset = GetBufferOffset(CellTarget, CellClipmap, CellVoxel);
					
					FFieldEvalContext EvalContext;
					SampleFieldData(SamplePosition, CellTarget,
						ClipmapExtent * 2 * 256 * 4 / ClipmapResolution,  ClipmapDistance * 8 / ClipmapResolution, TimeSeconds, NodesParams, NodesOffsets, TargetsOffsets, EvalContext);
					
					const int AttributeOffset = ClipmapResolution * ClipmapResolution * ClipmapResolution * ClipmapCount;
					if (EvalContext.DataIndex  == 3)
					{
						FieldClipmap[BufferOffset] = GetFieldValue(EvalContext, 0) * ClipmapFade;
						FieldClipmap[BufferOffset + 1 * AttributeOffset] = GetFieldValue(EvalContext, 1) * ClipmapFade;
						FieldClipmap[BufferOffset + 2 * AttributeOffset] = GetFieldValue(EvalContext, 2) * ClipmapFade;
					}
					else if (EvalContext.DataIndex == 1)
					{
						FieldClipmap[BufferOffset] = GetFieldValue(EvalContext, 0) * ClipmapFade;
					}
				}
			}
		}
	}
}