// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
    PhysicsFieldEval.ush
=============================================================================*/
#pragma once
#include "Common.ush"

/* -----------------------------------------------------------------
 * Eval context
 * -----------------------------------------------------------------
 */

#define MAX_DATA_SIZE 16
struct FFieldEvalContext
{
	/** Data buffer that will represent the result of each node evaluation that could be consumed by other nodes */
    float NodeEvalResultBuffer[MAX_DATA_SIZE];
	
	/** Data current index in that data buffer */
    int DataIndex;
	
	/** Time in seconds since the creation */
	float TimeSeconds;
};

// Define here each individual names
#define PHYSICS_FIELD_NODES_PARAMS EvalContext_NodesParams
#define PHYSICS_FIELD_NODES_OFFSETS EvalContext_NodesOffsets
#define PHYSICS_FIELD_TARGETS_OFFSETS EvalContext_TargetsOffsets

// PS4/PS5 don't support Buffers within struct unfortunately. So need to pass the buffers individually
#define PHYSICS_FIELD_BUFFERS in Buffer<float> PHYSICS_FIELD_NODES_PARAMS, in Buffer<int> PHYSICS_FIELD_NODES_OFFSETS,  in Buffer<int> PHYSICS_FIELD_TARGETS_OFFSETS
#define PHYSICS_FIELD_ARGS PHYSICS_FIELD_NODES_PARAMS, PHYSICS_FIELD_NODES_OFFSETS, PHYSICS_FIELD_TARGETS_OFFSETS

float GetFieldValue(FFieldEvalContext EvalContext, int DataIndex)
{
    return (DataIndex < MAX_DATA_SIZE) ? EvalContext.NodeEvalResultBuffer[DataIndex] : 0;
}

float3 GetFieldVector(FFieldEvalContext EvalContext, int DataIndex)
{
	return ((DataIndex+2) < MAX_DATA_SIZE) ? float3(EvalContext.NodeEvalResultBuffer[DataIndex], EvalContext.NodeEvalResultBuffer[DataIndex+1], EvalContext.NodeEvalResultBuffer[DataIndex+2]) : float3(0,0,0);
}

void SetFieldValue(inout FFieldEvalContext EvalContext, int DataIndex, float FieldValue)
{
	if(DataIndex < MAX_DATA_SIZE)
	{
		EvalContext.NodeEvalResultBuffer[DataIndex] = FieldValue;
	}
}

void InitEvalContext(inout FFieldEvalContext EvalContext, in float TimeSeconds)
{
	[unroll]
	for (int DataIndex = 0; DataIndex < MAX_DATA_SIZE; ++DataIndex)
	{
		EvalContext.NodeEvalResultBuffer[DataIndex] = 0.0;
	}
    EvalContext.DataIndex = 0;
	EvalContext.TimeSeconds = TimeSeconds;
}

/* -----------------------------------------------------------------
 * Perlin noise
 * -----------------------------------------------------------------
 */

float3 ComputeHashValue(uint3 v)
{
	// PCG Step
	v = v * 1664525u + 1013904223u;
	v.x += v.y * v.z; v.y += v.z * v.x; v.z += v.x * v.y;
	v ^= v >> 16u;
	v.x += v.y * v.z; v.y += v.z * v.x; v.z += v.x * v.y;
    
	// Efficiently map to [-1.0, 1.0]
	// 0xFFFFFFu is the mask for 24-bit precision (mantissa of a float)
	return (float3(v & 0xFFFFFFu) / float(0xFFFFFFu)) * 2.0 - 1.0;
}

float ComputePerlinNoise(float3 pos)
{
	float3 floorPos = floor(pos);
	float3 f = frac(pos);
    
	// Use uint3 for the base grid coordinate
	uint3 uPos = uint3(abs(floorPos)); // abs handles negative world coords safely

	// Quintic interpolation
	float3 u = f * f * f * (f * (f * 6.0 - 15.0) + 10.0);

	// Evaluate gradients at 8 corners using direct uint3 offsets
	// This avoids all casting inside the hot loop
	float n000 = dot(ComputeHashValue(uPos + uint3(0,0,0)), f - float3(0,0,0));
	float n100 = dot(ComputeHashValue(uPos + uint3(1,0,0)), f - float3(1,0,0));
	float n010 = dot(ComputeHashValue(uPos + uint3(0,1,0)), f - float3(0,1,0));
	float n110 = dot(ComputeHashValue(uPos + uint3(1,1,0)), f - float3(1,1,0));
	float n001 = dot(ComputeHashValue(uPos + uint3(0,0,1)), f - float3(0,0,1));
	float n101 = dot(ComputeHashValue(uPos + uint3(1,0,1)), f - float3(1,0,1));
	float n011 = dot(ComputeHashValue(uPos + uint3(0,1,1)), f - float3(0,1,1));
	float n111 = dot(ComputeHashValue(uPos + uint3(1,1,1)), f - float3(1,1,1));

	// Nested Lerps for trilinear interpolation
	return lerp(
		lerp(lerp(n000, n100, u.x), lerp(n010, n110, u.x), u.y),
		lerp(lerp(n001, n101, u.x), lerp(n011, n111, u.x), u.y), 
		u.z
	);
}

/* -----------------------------------------------------------------
 * Random Vector
 * -----------------------------------------------------------------
 */
uint3 ComputeRandomVector(int3 pos)
{
	// taking a signed int then reinterpreting as unsigned gives good behavior for negatives
	uint3 v = uint3(pos);

	// Linear congruential step. These LCG constants are from Numerical Recipies
	// For additional #'s, PCG would do multiple LCG steps and scramble each on output
	// So v here is the RNG state
	v = v * 1664525u + 1013904223u;

	// PCG uses xorshift for the final shuffle, but it is expensive (and cheap
	// versions of xorshift have visible artifacts). Instead, use simple MAD Feistel steps
	//
	// Feistel ciphers divide the state into separate parts (usually by bits)
	// then apply a series of permutation steps one part at a time. The permutations
	// use a reversible operation (usually ^) to part being updated with the result of
	// a permutation function on the other parts and the key.
	//
	// In this case, I'm using v.x, v.y and v.z as the parts, using + instead of ^ for
	// the combination function, and just multiplying the other two parts (no key) for 
	// the permutation function.
	//
	// That gives a simple mad per round.
	v.x += v.y * v.z;
	v.y += v.z * v.x;
	v.z += v.x * v.y;
	v.x += v.y * v.z;
	v.y += v.z * v.x;
	v.z += v.x * v.y;

	// only top 16 bits are well shuffled
	return v >> 16u;
}

/* -----------------------------------------------------------------
 * Field System defines
 * -----------------------------------------------------------------
 */

#define NONE_TYPE 0
#define RESULTS_TYPE 1
#define INTEGER_TYPE 2
#define SCALAR_TYPE 3
#define VECTOR_TYPE 4

#define NONE_TARGET 0
#define DYNAMIC_STATE 1
#define LINEAR_FORCE 2
#define EXTERNAL_STRAIN    3
#define KILL_PARTICLES 4
#define LINEAR_VELOCITY 5
#define ANGULAR_VELOCITY 6
#define ANGULAR_TORQUE 7
#define INTERNAL_STRAIN 8
#define DISABLED_THRESHOLD 9
#define SLEEPING_THRESHOLD 10
#define POSITION_STATIC 11
#define POSITION_ANIMATED 12
#define POSITION_TARGET 13
#define DYNAMIC_CONSTRAINT 14
#define COLLISION_GROUP 15
#define ACTIVATE_DISABLED 16

#define NONE_NODE 0
#define UNIFORM_INTEGER 1
#define RADIAL_MASK_INTEGER 2
#define UNIFORM_SCALAR 3
#define RADIAL_FALLOFF_SCALAR 4
#define PLANE_FALLOFF_SCALAR 5
#define BOX_FALLOFF_SCALAR 6
#define NOISE_SCALAR 7
#define UNIFORM_VECTOR 8
#define RADIAL_VECTOR 9
#define RANDOM_VECTOR 10
#define SUM_SCALAR 11
#define SUM_VECTOR 12
#define CONVERSION_FIELD 13
#define CULLING_FIELD 14
#define WAVE_SCALAR 15

#define SET_ALWAYS 0
#define SET_IFF_NOT_INTERIOR 1
#define SET_IFF_NOT_EXTERIOR 2

#define NONE_FALLOFF 0
#define FALLOFF_LINEAR 1
#define FALLOFF_INVERSE 2
#define FALLOFF_SQUARED 3
#define FALLOFF_LOGARITHMIC 4

#define MULTIPLY_OP 0
#define DIVIDE_OP 1
#define ADD_OP 2
#define SUBTRACT_OP 3

#define CULLING_INSIDE 0
#define CULLING_OUTSIDE 1

#define WAVE_COSINE 0
#define WAVE_GAUSSIAN 1
#define WAVE_FALLOFF 2
#define WAVE_DECAY 3

/* -----------------------------------------------------------------
 * Quat utils
 * -----------------------------------------------------------------
 */

float3 RotateVectorByQuat(in float3 Vector, in float4 Quat)
{
	float3 T = 2.0 * cross(Quat.xyz, Vector);
	return Vector + Quat.w * T + cross(Quat.xyz, T);
}

float3 UnRotateVectorByQuat(in float3 Vector, in float4 Quat)
{
	float3 T = 2.0 * cross(Quat.xyz, Vector);
	return Vector - Quat.w * T + cross(Quat.xyz, T);
}

/* -----------------------------------------------------------------
 * Common Falloff functions
 * -----------------------------------------------------------------
 */

float ComputeFalloffValue(int FieldFalloff, float FieldMagnitude, float DeltaDistance)
{
	float Result = 0.0;
    
	// Modern HLSL compilers handle switch statements very efficiently
	switch (FieldFalloff)
	{
		case NONE_FALLOFF: 
			Result = 1.0;
			break;
		case FALLOFF_LINEAR: 
			Result = DeltaDistance;
			break;
		case 3: 
			Result = DeltaDistance * DeltaDistance;
			break;
		case FALLOFF_INVERSE: 
			Result = 2.0 * (1.0 - 1.0 / (DeltaDistance + 1.0));
			break;
		case FALLOFF_LOGARITHMIC: 
			// log2(x) is a native hardware instruction, much faster than log(x)/log(2)
			Result = log2(DeltaDistance + 1.0);
			break;
		default:
			break;
	}
    
	return Result * FieldMagnitude;
}

/* -----------------------------------------------------------------
 * Uniform Integer Field
 * -----------------------------------------------------------------
 */

#define UNIFORM_INTEGER_MAGNITUDE 0

void EvaluateUniformInteger(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    SetFieldValue(EvalContext, EvalContext.DataIndex++, PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_INTEGER_MAGNITUDE]);
}

/* -----------------------------------------------------------------
 * Radial Int Mask Field
 * -----------------------------------------------------------------
 */

#define RADIAL_MASK_INTEGER_RADIUS 0
#define RADIAL_MASK_INTEGER_POSITIONX 1
#define RADIAL_MASK_INTEGER_POSITIONY 2
#define RADIAL_MASK_INTEGER_POSITIONZ 3
#define RADIAL_MASK_INTEGER_INTERIOR_VALUE 4
#define RADIAL_MASK_INTEGER_EXTERIOR_VALUE 5
#define RADIAL_MASK_INTEGER_SET_MASK_CONDITION 6

void EvaluateRadialMaskInteger(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    const float MaskRadius = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_RADIUS];
    const float3 CenterPosition = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_POSITIONX], 
    									 PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_POSITIONY], 
    									 PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_POSITIONZ]);
    const int InteriorValue = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_INTERIOR_VALUE];
    const int ExteriorValue = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_EXTERIOR_VALUE];
    const int MaskCondition = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_MASK_INTEGER_SET_MASK_CONDITION];
    
    const float3 DeltaPosition = SamplePosition - CenterPosition;
    const float DistanceSquared = dot(DeltaPosition, DeltaPosition);
    const float RadiusSquared = MaskRadius * MaskRadius;
    
    const int DeltaResult = DistanceSquared < RadiusSquared ? InteriorValue : ExteriorValue;
    const int CurrentValue = (int)GetFieldValue(EvalContext, EvalContext.DataIndex);
    
    const float ResultScalar = (MaskCondition == SET_ALWAYS) ? DeltaResult :
        ((MaskCondition == SET_IFF_NOT_INTERIOR && CurrentValue != InteriorValue) ? DeltaResult :
        ((MaskCondition == SET_IFF_NOT_EXTERIOR && CurrentValue != ExteriorValue) ? DeltaResult : CurrentValue));
    
    SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultScalar);
}

/* -----------------------------------------------------------------
 * Uniform Scalar Field
 * -----------------------------------------------------------------
 */

#define UNIFORM_SCALAR_MAGNITUDE 0

void EvaluateUniformScalar(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    SetFieldValue(EvalContext, EvalContext.DataIndex++, PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_SCALAR_MAGNITUDE]);
}

/* -----------------------------------------------------------------
 * Wave Scalar Field
 * -----------------------------------------------------------------
 */

#define WAVE_SCALAR_MAGNITUDE 0
#define WAVE_SCALAR_POSITIONX 1
#define WAVE_SCALAR_POSITIONY 2
#define WAVE_SCALAR_POSITIONZ 3
#define WAVE_SCALAR_WAVELENGTH 4
#define WAVE_SCALAR_PERIOD 5
#define WAVE_SCALAR_TIME 6
#define WAVE_SCALAR_FUNCTION 7
#define WAVE_SCALAR_FALLOFF 8

static const float WAVE_PI = 3.14159265f;

void EvaluateWaveScalar(float3 SamplePosition, int NodeOffset, float MinLength, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_MAGNITUDE];
    const float3 CenterPosition = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_POSITIONX], 
    									 PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_POSITIONY], 
    									 PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_POSITIONZ]);
	
    const float Wavelength = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_WAVELENGTH];
    const float WavePeriod = max(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_PERIOD], 0.0001);
    const float StartTime = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_TIME];
    const int WaveFunction = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_FUNCTION];
    const int FalloffType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + WAVE_SCALAR_FALLOFF];
	
    const float DistanceValue = length(SamplePosition - CenterPosition);
	
	const float WaveVelocity = Wavelength * rcp(WavePeriod);
	const float WaveNumber = 2.0 * WAVE_PI * rcp(max(Wavelength, MinLength));
	const float DeltaTime = max(EvalContext.TimeSeconds - StartTime, 0.0);
	const float WaveRadius = WaveVelocity * DeltaTime;
	const float WavePhase = WaveNumber * (DistanceValue - WaveRadius);
	
	float ResultScalar = 0.0f;
	switch (WaveFunction)
	{
		case WAVE_GAUSSIAN:
			ResultScalar = Magnitude * exp(-WavePhase * WavePhase);
			break;
		case WAVE_COSINE:
			ResultScalar = Magnitude * cos(WavePhase);
			break;
		case WAVE_FALLOFF:
			{
				const bool bHasFalloff = (DistanceValue < WaveRadius) && (WaveRadius > 0.0);
				ResultScalar = bHasFalloff ? ComputeFalloffValue(FalloffType, Magnitude, (1.0 - DistanceValue * rcp(WaveRadius))) : 0.0;
				break;
			}
		case WAVE_DECAY:
			{
				const float WaveDecay = DeltaTime * rcp(WavePeriod);
				ResultScalar = Magnitude * exp(-WaveDecay * WaveDecay);
				break;
			}
		default : break;
	}
    
    SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultScalar);
}

/* -----------------------------------------------------------------
 * Radial Falloff Field
 * -----------------------------------------------------------------
 */

#define RADIAL_FALLOFF_MAGNITUDE 0
#define RADIAL_FALLOFF_MIN_RANGE 1
#define RADIAL_FALLOFF_MAX_RANGE 2
#define RADIAL_FALLOFF_DEFAULT 3
#define RADIAL_FALLOFF_RADIUS 4
#define RADIAL_FALLOFF_POSITIONX 5
#define RADIAL_FALLOFF_POSITIONY 6
#define RADIAL_FALLOFF_POSITIONZ 7
#define RADIAL_FALLOFF_FALLOFF 8

void EvaluateRadialFalloffScalar(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_MAGNITUDE];
	const float MinRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_MIN_RANGE];
	const float MaxRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_MAX_RANGE];
	const float DefaultValue = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_DEFAULT];
	const float FalloffRadius = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_RADIUS];
	
	const float3 CenterPosition = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_POSITIONX], 
		PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_POSITIONY], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_POSITIONZ]);
	const int FalloffType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_FALLOFF_FALLOFF];
	
	const float3 DeltaPosition = SamplePosition - CenterPosition;
	const float DistanceSquared = dot(DeltaPosition, DeltaPosition);
	const float RadiusSquared = FalloffRadius * FalloffRadius;
    
	float ResultScalar = DefaultValue;
	if (RadiusSquared > 0.0 && DistanceSquared < RadiusSquared)
	{
		const float FalloffValue = ComputeFalloffValue(FalloffType, 1.0, 1.0 - sqrt(DistanceSquared) * rcp(FalloffRadius));
		ResultScalar = Magnitude * (MinRange + FalloffValue * (MaxRange - MinRange));
	}
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultScalar);
}

/* -----------------------------------------------------------------
 * Plane Falloff Field
 * -----------------------------------------------------------------
 */

#define PLANE_FALLOFF_MAGNITUDE 0
#define PLANE_FALLOFF_MIN_RANGE 1
#define PLANE_FALLOFF_MAX_RANGE 2
#define PLANE_FALLOFF_DEFAULT 3
#define PLANE_FALLOFF_DISTANCE 4
#define PLANE_FALLOFF_POSITIONX 5
#define PLANE_FALLOFF_POSITIONY 6
#define PLANE_FALLOFF_POSITIONZ 7
#define PLANE_FALLOFF_NORMALX 8
#define PLANE_FALLOFF_NORMALY 9
#define PLANE_FALLOFF_NORMALZ 10
#define PLANE_FALLOFF_FALLOFF 11

void EvaluatePlaneFalloffScalar(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_MAGNITUDE];
	const float MinRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_MIN_RANGE];
	const float MaxRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_MAX_RANGE];
	const float DefaultValue = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_DEFAULT];
	const float PlaneDistance = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_DISTANCE];
	
	const float3 PlanePosition = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_POSITIONX], 
									    PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_POSITIONY], 
									    PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_POSITIONZ]);
	
	const float3 PlaneNormal = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_NORMALX], 
									  PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_NORMALY], 
									  PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_NORMALZ]);
	
	const int FalloffType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + PLANE_FALLOFF_FALLOFF];
	
	const float PlaneOffset = dot(PlanePosition, PlaneNormal);
	const float LocalDistance = dot(PlaneNormal, SamplePosition) - PlaneOffset;
    
	float ResultScalar = DefaultValue;
	if (PlaneDistance > 0.0 && LocalDistance > -PlaneDistance && LocalDistance < 0.0)
	{
		const float InverseDistance = rcp(PlaneDistance);
		const float FalloffValue = ComputeFalloffValue(FalloffType, 1.0, 1.0 + LocalDistance * InverseDistance);
		ResultScalar = Magnitude * (MinRange + FalloffValue * (MaxRange - MinRange));
	}
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultScalar);
}

/* -----------------------------------------------------------------
 * Box Falloff Field
 * -----------------------------------------------------------------
 */

#define BOX_FALLOFF_MAGNITUDE 0
#define BOX_FALLOFF_MIN_RANGE 1
#define BOX_FALLOFF_MAX_RANGE 2
#define BOX_FALLOFF_DEFAULT 3
#define BOX_FALLOFF_ROTATIONX 4
#define BOX_FALLOFF_ROTATIONY 5
#define BOX_FALLOFF_ROTATIONZ 6
#define BOX_FALLOFF_ROTATIONW 7
#define BOX_FALLOFF_TRANSLATIONX 8
#define BOX_FALLOFF_TRANSLATIONY 9
#define BOX_FALLOFF_TRANSLATIONZ 10
#define BOX_FALLOFF_SCALEX 11
#define BOX_FALLOFF_SCALEY 12
#define BOX_FALLOFF_SCALEZ 13
#define BOX_FALLOFF_FALLOFF 14

void EvaluateBoxFalloffScalar(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_MAGNITUDE];
	const float MinRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_MIN_RANGE];
	const float MaxRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_MAX_RANGE];
	const float DefaultValue = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_DEFAULT];
	
	const float4 BoxRotation = float4(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_ROTATIONX], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_ROTATIONY],
							          PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_ROTATIONZ], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_ROTATIONW]);

	const float3 BoxTranslation = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_TRANSLATIONX],
									     PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_TRANSLATIONY],
										 PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_TRANSLATIONZ]);

	const float3 BoxScale = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_SCALEX],
								   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_SCALEY],
								   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_SCALEZ]);
	
	const int FalloffType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + BOX_FALLOFF_FALLOFF];
	
	const float3 InverseScale = rcp(max(BoxScale, 0.0001));
	const float3 LocalPosition = UnRotateVectorByQuat(SamplePosition - BoxTranslation, BoxRotation) * InverseScale;
	const float3 DeltaPosition = abs(LocalPosition) - 50.0;
	
	const float MaxComponent = max(max(DeltaPosition.x, DeltaPosition.y), DeltaPosition.z);
	const float OutsideDistance = length(max(DeltaPosition, 0.0));
	const float LocalDistance = OutsideDistance + min(MaxComponent, 0.0);
    
	float ResultScalar = DefaultValue;
	if (LocalDistance < 0.0)
	{
		const float FalloffValue = ComputeFalloffValue(FalloffType, 1.0, -LocalDistance * 0.02);
		ResultScalar = Magnitude * (MinRange + FalloffValue * (MaxRange - MinRange));
	}
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultScalar);
}

/* -----------------------------------------------------------------
 * Noise Field
 * -----------------------------------------------------------------
 */

#define NOISE_MIN_RANGE 0
#define NOISE_MAX_RANGE 1
#define NOISE_ROTATIONX 2
#define NOISE_ROTATIONY 3
#define NOISE_ROTATIONZ 4
#define NOISE_ROTATIONW 5
#define NOISE_TRANSLATIONX 6
#define NOISE_TRANSLATIONY 7
#define NOISE_TRANSLATIONZ 8
#define NOISE_SCALEX 9
#define NOISE_SCALEY 10
#define NOISE_SCALEZ 11

void EvaluateNoiseScalar(float3 SamplePosition, int NodeOffset, float MinScale, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float MinRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_MIN_RANGE];
	const float MaxRange = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_MAX_RANGE];
	
	const float4 NoiseRotation = float4(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_ROTATIONX], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_ROTATIONY],
									    PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_ROTATIONZ], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_ROTATIONW]);
	
	const float3 NoiseTranslation = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_TRANSLATIONX], 
										   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_TRANSLATIONY], 
										   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_TRANSLATIONZ]);
	
	const float3 NoiseScale = 0.5 * max(float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_SCALEX], 
											   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_SCALEY],  
											   PHYSICS_FIELD_NODES_PARAMS[NodeOffset + NOISE_SCALEZ]), MinScale);
    
	const float3 InverseScale = rcp(max(NoiseScale, 0.0001));
	float3 LocalPosition = fmod(UnRotateVectorByQuat(SamplePosition - NoiseTranslation, NoiseRotation), NoiseScale) * InverseScale;
	LocalPosition = clamp(LocalPosition, -1.0, 1.0) * 127.5 + 127.5;
    
	const float NoiseValue = ComputePerlinNoise(LocalPosition) * 0.5 + 0.5;
	const float DeltaRange = MaxRange - MinRange;
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, NoiseValue * DeltaRange + MinRange);
}

/* -----------------------------------------------------------------
 * Uniform Vector Field
 * -----------------------------------------------------------------
 */

#define UNIFORM_VECTOR_MAGNITUDE 0
#define UNIFORM_VECTOR_DIRECTIONX 1
#define UNIFORM_VECTOR_DIRECTIONY 2
#define UNIFORM_VECTOR_DIRECTIONZ 3

void EvaluateUniformVector(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_VECTOR_MAGNITUDE];
	const float3 Direction = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_VECTOR_DIRECTIONX], 
		PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_VECTOR_DIRECTIONY], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + UNIFORM_VECTOR_DIRECTIONZ]);
	
	const float3 ResultVector = Magnitude * Direction;
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.x);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.y);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.z);
}

/* -----------------------------------------------------------------
 * Utils to set a radial direction
 * -----------------------------------------------------------------
 */

void SetRadialDirection(float3 RadialDirection, float Magnitude, inout FFieldEvalContext EvalContext)
{
	const float LengthSquared = dot(RadialDirection, RadialDirection);
    
	// Use rsqrt for normalization
	const float InverseLength = LengthSquared > 0.0 ? rsqrt(LengthSquared) : 0.0;
	const float3 ResultVector = Magnitude * RadialDirection * InverseLength;
    
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.x);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.y);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.z);
}

/* -----------------------------------------------------------------
 * Radial Vector Field
 * -----------------------------------------------------------------
 */

#define RADIAL_VECTOR_MAGNITUDE 0
#define RADIAL_VECTOR_POSITIONX 1
#define RADIAL_VECTOR_POSITIONY 2
#define RADIAL_VECTOR_POSITIONZ 3

void EvaluateRadialVector(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_VECTOR_MAGNITUDE];
	const float3 Center = float3(PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_VECTOR_POSITIONX], 
		PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_VECTOR_POSITIONY], PHYSICS_FIELD_NODES_PARAMS[NodeOffset + RADIAL_VECTOR_POSITIONZ]);
    
	const float3 RadialDirection = SamplePosition - Center;
	SetRadialDirection(RadialDirection, Magnitude, EvalContext);
}

/* -----------------------------------------------------------------
 * Random Vector Field
 * -----------------------------------------------------------------
 */

#define RANDOM_VECTOR_MAGNITUDE 0

void EvaluateRandomVector(float3 SamplePosition, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset+RANDOM_VECTOR_MAGNITUDE];
	
	const float3 RadialDirection = float3(ComputeRandomVector(int3(SamplePosition))) * rcp(65535.0) - 0.5;
	SetRadialDirection(RadialDirection, Magnitude, EvalContext);
}

/* -----------------------------------------------------------------
 * Sum Scalar Field
 * -----------------------------------------------------------------
 */

#define SUM_SCALAR_MAGNITUDE 0
#define SUM_SCALAR_RIGHT 1
#define SUM_SCALAR_LEFT 2
#define SUM_SCALAR_OPERATION 3

void EvaluateSumScalar(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_SCALAR_MAGNITUDE];
	const int HasRight = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_SCALAR_RIGHT];
	const int HasLeft = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_SCALAR_LEFT];
	const int Operation = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_SCALAR_OPERATION];
    
	EvalContext.DataIndex -= HasRight + HasLeft;
	
	const float DefaultValue = (Operation == MULTIPLY_OP || Operation == DIVIDE_OP) ? 1.0 : 0.0;
	const float RightValue = HasRight ? GetFieldValue(EvalContext, EvalContext.DataIndex) : DefaultValue;
	const float LeftValue = HasLeft ? GetFieldValue(EvalContext, EvalContext.DataIndex + HasRight) : DefaultValue;
	
	float ResultValue = 0.0;
	switch (Operation)
	{
		case MULTIPLY_OP: 
			ResultValue = RightValue * LeftValue;
			break;
		case DIVIDE_OP:
			ResultValue = LeftValue * rcp(RightValue);
			break;
		case ADD_OP: 
			ResultValue = RightValue + LeftValue;
			break;
		case SUBTRACT_OP:
			ResultValue = LeftValue - RightValue;
			break;
		default : break;
	}
	SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultValue * Magnitude);
}

/* -----------------------------------------------------------------
 * Sum Vector Field
 * -----------------------------------------------------------------
 */

#define SUM_VECTOR_MAGNITUDE 0
#define SUM_VECTOR_SCALAR 1
#define SUM_VECTOR_RIGHT 2
#define SUM_VECTOR_LEFT 3
#define SUM_VECTOR_OPERATION 4

void EvaluateSumVector(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    const float Magnitude = PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_VECTOR_MAGNITUDE];
    const int HasScalar = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_VECTOR_SCALAR];
    const int HasRight = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_VECTOR_RIGHT];
    const int HasLeft = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_VECTOR_LEFT];
    const int Operation = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + SUM_VECTOR_OPERATION];
    
    EvalContext.DataIndex -= HasScalar + HasRight * 3 + HasLeft * 3;
	
    const float3 DefaultVec = (Operation == MULTIPLY_OP || Operation == DIVIDE_OP) ? 1.0 : 0.0;
    const float ScalarValue = HasScalar ? GetFieldValue(EvalContext, EvalContext.DataIndex) : 1.0;
	
    const float3 RightVector = HasRight ? GetFieldVector(EvalContext, EvalContext.DataIndex + HasScalar) : DefaultVec;
    const float3 LeftVector = HasLeft ? GetFieldVector(EvalContext, EvalContext.DataIndex + HasScalar + HasRight * 3) : DefaultVec;
	
	float3 ResultVector = 0.0;
	switch (Operation)
	{
		case MULTIPLY_OP: 
			ResultVector = RightVector * LeftVector;
			break;
		case DIVIDE_OP:
			ResultVector = LeftVector * rcp(RightVector);
			break;
		case ADD_OP: 
			ResultVector = RightVector + LeftVector;
			break;
		case SUBTRACT_OP:
			ResultVector = LeftVector - RightVector;
			break;
		default : break;
	}
    
    ResultVector *= ScalarValue * Magnitude;
    SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.x);
    SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.y);
    SetFieldValue(EvalContext, EvalContext.DataIndex++, ResultVector.z);
}

/* -----------------------------------------------------------------
 * Culling Scalar
 * -----------------------------------------------------------------
 */

#define CULLING_SCALAR 0
#define CULLING_INPUT 1
#define CULLING_OPERATION 2

void EvaluateCullingScalar(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const int HasScalar = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_SCALAR];
	const int HasInput = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_INPUT];
	const int Operation = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_OPERATION];
    
	EvalContext.DataIndex -= HasScalar + HasInput;
    
	const float CullingScalar = HasScalar ? GetFieldValue(EvalContext, EvalContext.DataIndex) : 0.0;
	const bool ValidCulling = (Operation == CULLING_OUTSIDE && CullingScalar != 0.0) || 
							  (Operation == CULLING_INSIDE && CullingScalar == 0.0);
	
	const float InputScalar = (HasInput && ValidCulling) ? GetFieldValue(EvalContext, EvalContext.DataIndex + HasScalar) : 0.0;
	SetFieldValue(EvalContext, EvalContext.DataIndex++, InputScalar);
}

/* -----------------------------------------------------------------
 * Culling Vector
 * -----------------------------------------------------------------
 */

void EvaluateCullingVector(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	const int HasScalar = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_SCALAR];
	const int HasInput = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_INPUT];
	const int Operation = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + CULLING_OPERATION];
    
	EvalContext.DataIndex -= HasScalar + HasInput * 3;
    
	const float CullingScalar = HasScalar ? GetFieldValue(EvalContext, EvalContext.DataIndex) : 0.0;
	const bool ValidCulling = (Operation == CULLING_OUTSIDE && CullingScalar != 0.0) || 
							  (Operation == CULLING_INSIDE && CullingScalar == 0.0);
	
	const float3 InputVector = (HasInput && ValidCulling) ? GetFieldVector(EvalContext, EvalContext.DataIndex + HasScalar) : 0.0;
	SetFieldValue(EvalContext, EvalContext.DataIndex++, InputVector.x);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, InputVector.y);
	SetFieldValue(EvalContext, EvalContext.DataIndex++, InputVector.z);
}

/* -----------------------------------------------------------------
 * Culling Integer
 * -----------------------------------------------------------------
 */

void EvaluateCullingInteger(int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
	EvaluateCullingScalar(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
}

/* -----------------------------------------------------------------
 * Main Evaluation Dispatchers - Using switch for better performance
 * -----------------------------------------------------------------
 */

void EvaluateFieldNodeScalar(float3 SamplePosition, int NodeType, int NodeOffset, float MinScale, float MinLength, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    switch(NodeType)
    {
        case UNIFORM_SCALAR:
			EvaluateUniformScalar(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case WAVE_SCALAR:
			EvaluateWaveScalar(SamplePosition, NodeOffset, MinLength, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case RADIAL_FALLOFF_SCALAR:
			EvaluateRadialFalloffScalar(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case PLANE_FALLOFF_SCALAR:
			EvaluatePlaneFalloffScalar(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case BOX_FALLOFF_SCALAR:
			EvaluateBoxFalloffScalar(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case NOISE_SCALAR:
			EvaluateNoiseScalar(SamplePosition, NodeOffset, MinScale, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case SUM_SCALAR:
			EvaluateSumScalar(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case CONVERSION_FIELD:
            // No-op: conversion happens in-place
            break;
        case CULLING_FIELD:
			EvaluateCullingScalar(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
		default : break;
    }
}

void EvaluateFieldNodeInteger(float3 SamplePosition, int NodeType, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    switch(NodeType)
    {
        case UNIFORM_INTEGER:
			EvaluateUniformInteger(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case RADIAL_MASK_INTEGER:
			EvaluateRadialMaskInteger(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case CONVERSION_FIELD:
            // No-op: conversion happens in-place
            break;
        case CULLING_FIELD:
			EvaluateCullingInteger(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        default : break;
    }
}

void EvaluateFieldNodeVector(float3 SamplePosition, int NodeType, int NodeOffset, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    switch(NodeType)
    {
        case UNIFORM_VECTOR:
			EvaluateUniformVector(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case RADIAL_VECTOR:
			EvaluateRadialVector(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case RANDOM_VECTOR:
			EvaluateRandomVector(SamplePosition, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case SUM_VECTOR:
			EvaluateSumVector(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case CULLING_FIELD:
			EvaluateCullingVector(NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
    	default : break;
    }
}

void EvaluateFieldNode(float3 SamplePosition, int NodeType, int NodeOffset, int DataType, float MinScale, float MinLength, inout FFieldEvalContext EvalContext, PHYSICS_FIELD_BUFFERS)
{
    switch(DataType)
    {
        case SCALAR_TYPE:
			EvaluateFieldNodeScalar(SamplePosition, NodeType, NodeOffset, MinScale, MinLength, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case INTEGER_TYPE:
			EvaluateFieldNodeInteger(SamplePosition, NodeType, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
        case VECTOR_TYPE:
			EvaluateFieldNodeVector(SamplePosition, NodeType, NodeOffset, EvalContext, PHYSICS_FIELD_ARGS);
            break;
		default : break;
    }
}

void SampleFieldData(float3 SamplePosition, int TargetType, float MinScale, float MinLength, float TimeSeconds,
	PHYSICS_FIELD_BUFFERS, inout FFieldEvalContext EvalContext)
{
	InitEvalContext(EvalContext, TimeSeconds);
    
    const int NodesBegin = PHYSICS_FIELD_TARGETS_OFFSETS[TargetType];
    const int NodesEnd = PHYSICS_FIELD_TARGETS_OFFSETS[TargetType + 1];
    
    // Loop through nodes
    for (int NodeIndex = NodesBegin; NodeIndex < NodesEnd; ++NodeIndex)
    {
        const int NodeOffset = PHYSICS_FIELD_NODES_OFFSETS[NodeIndex];
        const int DataType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset];
        const int NodeType = (int)PHYSICS_FIELD_NODES_PARAMS[NodeOffset + 1];
    	
		EvaluateFieldNode(SamplePosition, NodeType, NodeOffset + 2, DataType, MinScale, MinLength, EvalContext, PHYSICS_FIELD_ARGS);
	}
}

/* -----------------------------------------------------------------
 * Public API
 * -----------------------------------------------------------------
 */

float3 PhysicsField_EvalPhysicsVectorField(float3 SamplePosition, int TargetIndex, PHYSICS_FIELD_BUFFERS, in float TimeSeconds)
{
    FFieldEvalContext EvalContext;
	SampleFieldData(SamplePosition, TargetIndex, 0.0, 0.0, TimeSeconds, PHYSICS_FIELD_ARGS, EvalContext);
    
    return EvalContext.DataIndex == 3 ? float3(GetFieldValue(EvalContext, 0), GetFieldValue(EvalContext, 1), GetFieldValue(EvalContext, 2)) : 0.0;
}

float PhysicsField_EvalPhysicsScalarField(float3 SamplePosition, int TargetIndex, PHYSICS_FIELD_BUFFERS, in float TimeSeconds)
{
    FFieldEvalContext EvalContext;
	SampleFieldData(SamplePosition, TargetIndex, 0.0, 0.0, TimeSeconds, PHYSICS_FIELD_ARGS, EvalContext);
    
    return EvalContext.DataIndex == 1 ? GetFieldValue(EvalContext, 0) : 0.0;
}

int PhysicsField_EvalPhysicsIntegerField(float3 SamplePosition, int TargetIndex, PHYSICS_FIELD_BUFFERS, in float TimeSeconds)
{
    FFieldEvalContext EvalContext;
	SampleFieldData(SamplePosition, TargetIndex, 0.0, 0.0, TimeSeconds, PHYSICS_FIELD_ARGS, EvalContext);
    
    return EvalContext.DataIndex == 1 ? (int)GetFieldValue(EvalContext, 0) : 0;
}