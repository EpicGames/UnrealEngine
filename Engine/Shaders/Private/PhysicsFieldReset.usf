// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsFieldBuilder.ush
=============================================================================*/

#pragma once

#include "Common.ush"

/* -----------------------------------------------------------------
 * Field System constants and context
 * -----------------------------------------------------------------
 */

RWBuffer<float> FieldClipmap;
int NumCells;

Buffer<int> CellsOffsets;
Buffer<uint4> CellsMin;
Buffer<uint4> CellsMax;

int ClipmapCount;
int ClipmapResolution;

int TargetCount;
int TargetsIndex[MAX_PHYSICS_FIELD_TARGETS];

int ValidCount;
int ValidTargets[MAX_PHYSICS_FIELD_TARGETS];

/* -----------------------------------------------------------------
 * Reset Field system
 * -----------------------------------------------------------------
 */

//[numthreads(BUILD_FIELD_THREAD_GROUP_SIZE, 1, 1)]
//void ResetPhysicsFieldClipmapCS(
//	uint3 GroupId : SV_GroupID,
//	uint3 DispatchThreadId : SV_DispatchThreadID,
//    uint3 GroupThreadId : SV_GroupThreadID)
//{
//	if (DispatchThreadId.x < NumCells)
//	{
//		FieldClipmap[DispatchThreadId.x] = 0.0;
//	}
//}

#define NONE_TARGET 0
#define DYNAMIC_STATE 1
#define	LINEAR_FORCE 2
#define	EXTERNAL_CLUSTER_STRAIN	3
#define	FIELD_KILL 4
#define	LINEAR_VELOCITY 5
#define	ANGULAR_VELOCITY 6
#define	ANGULAR_TORQUE 7
#define	INTERNAL_CLUSTER_STRAIN 8
#define	DISABLE_THRESHOLD 9
#define	SLEEPING_THRESHOLD 10
#define	POSITION_STATIC 11
#define	POSITION_ANIMATED 12
#define	POSITION_TARGET 13
#define	DYNAMIC_CONSTRAINT 14
#define	COLLISION_GROUP 15
#define	ACTIVATE_DISABLED 16

uint3 GetCellVoxel(in int ThreadId, in int CellIndex)
{
	const int LocalIndex = ThreadId - CellsOffsets[CellIndex];
	const uint4 CellsDelta = CellsMax[CellIndex] - CellsMin[CellIndex];
					
	const int IndexZ = LocalIndex / (CellsDelta.y * CellsDelta.x);
	const int IndexT = LocalIndex % (CellsDelta.y * CellsDelta.x);
	const int IndexY = IndexT / CellsDelta.x;
	const int IndexX = IndexT % CellsDelta.x;
	
	return CellsMin[CellIndex].xyz + uint3(IndexX, IndexY, IndexZ);
}

int GetBufferOffset(in int CellTarget, in int CellClipmap, in uint3 CellVoxel)
{
	const int DatasOffset = CellClipmap + TargetsIndex[CellTarget] * ClipmapCount;
	return (CellVoxel.x + CellVoxel.y * ClipmapResolution + (CellVoxel.z + ClipmapResolution * DatasOffset) * ClipmapResolution * ClipmapResolution);
}

[numthreads(BUILD_FIELD_THREAD_GROUP_SIZE, 1, 1)]
void ResetPhysicsFieldClipmapCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID)
{
	if (DispatchThreadId.x < NumCells)
	{
		int CellIndex = 0;
		
		int CellTarget = -1;
		for (int ValidIndex = 0; ValidIndex < ValidCount; ++ValidIndex, CellIndex += ClipmapCount)
		{
			if ((DispatchThreadId.x >= CellsOffsets[CellIndex]) && (DispatchThreadId.x < CellsOffsets[CellIndex + ClipmapCount]))
			{
				CellTarget = ValidTargets[ValidIndex];
				break;
			}
		}
		if (CellTarget != -1)
		{
			int CellClipmap = -1;
			for (int ClipmapIndex = 0; ClipmapIndex < ClipmapCount; ++ClipmapIndex, ++CellIndex)
			{
				if ((DispatchThreadId.x >= CellsOffsets[CellIndex]) && (DispatchThreadId.x < CellsOffsets[CellIndex + 1]))
				{
					CellClipmap = ClipmapIndex;
					break;
				}
			}
			if (CellClipmap != -1)
			{
				const uint3 CellVoxel = GetCellVoxel(DispatchThreadId.x, CellIndex);
		
				if (all(CellVoxel.xyz < int3(ClipmapResolution, ClipmapResolution, ClipmapResolution)))
				{
					const int BufferOffset = GetBufferOffset(CellTarget, CellClipmap, CellVoxel);
					const int AttributeOffset = ClipmapResolution * ClipmapResolution * ClipmapResolution * ClipmapCount;
					if (CellTarget == LINEAR_FORCE || CellTarget == LINEAR_VELOCITY || CellTarget == ANGULAR_VELOCITY || CellTarget == ANGULAR_TORQUE || CellTarget == POSITION_TARGET)
					{
						FieldClipmap[BufferOffset] = 0;
						FieldClipmap[BufferOffset + AttributeOffset] = 0;
						FieldClipmap[BufferOffset + 2 * AttributeOffset] = 0;
					}
					else
					{
						FieldClipmap[BufferOffset] = 0;
					}
				}
			}
		}
	}
}