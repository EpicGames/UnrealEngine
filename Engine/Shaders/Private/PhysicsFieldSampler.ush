// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsFieldSampler.ush
=============================================================================*/

#pragma once

#include "PhysicsFieldShared.ush"
#include "PhysicsFieldEval.ush"

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5

#if IS_MATERIAL_SHADER
    // for materials, these are in the view UB
	#define PhysicsFieldClipmapBuffer			View.PhysicsFieldClipmapBuffer
	#define PhysicsFieldNodesParams				View.PhysicsFieldNodesParams
	#define PhysicsFieldNodesOffsets			View.PhysicsFieldNodesOffsets
	#define PhysicsFieldTargetsOffsets			View.PhysicsFieldTargetsOffsets
	#define PhysicsFieldClipmapCenter			View.PhysicsFieldClipmapCenter
	#define PhysicsFieldClipmapDistance			View.PhysicsFieldClipmapDistance
	#define PhysicsFieldClipmapResolution		View.PhysicsFieldClipmapResolution
	#define PhysicsFieldClipmapExponent			View.PhysicsFieldClipmapExponent
	#define PhysicsFieldClipmapCount			View.PhysicsFieldClipmapCount
	#define PhysicsFieldTargetCount				View.PhysicsFieldTargetCount
	#define PhysicsFieldTargets					View.PhysicsFieldTargets
	#define PhysicsFieldTimeSeconds				View.GameTime
	#define PhysicsFieldTargetMappings			View.PhysicsFieldTargetMappings
#else
	Buffer<float> PhysicsFieldClipmapBuffer;
	Buffer<float> PhysicsFieldNodesParams;
	Buffer<int> PhysicsFieldNodesOffsets;
	Buffer<int> PhysicsFieldTargetsOffsets;
	float3		PhysicsFieldClipmapCenter;
	float		PhysicsFieldClipmapDistance;
	int			PhysicsFieldClipmapResolution;
	int			PhysicsFieldClipmapExponent;
	int			PhysicsFieldClipmapCount;
	int			PhysicsFieldTargetCount;
	int4		PhysicsFieldTargets[MAX_PHYSICS_FIELD_TARGETS]; // x (Vector), y (Scalar), z (Integer), w (Padding)
	float		PhysicsFieldTimeSeconds;
	int4		PhysicsFieldTargetMappings[MAX_PHYSICS_FIELD_TARGETS]; 
#endif

float3 MatPhysicsField_SamplePhysicsVectorField(float3 WorldPosition, int VectorTarget)
{
	return PhysicsField_SamplePhysicsVectorField(WorldPosition, VectorTarget, PhysicsFieldTargets, PhysicsFieldTargetCount, PhysicsFieldClipmapCenter, PhysicsFieldClipmapDistance,
				PhysicsFieldClipmapExponent, PhysicsFieldClipmapCount, PhysicsFieldClipmapResolution, PhysicsFieldClipmapBuffer);
}

float MatPhysicsField_SamplePhysicsScalarField(float3 WorldPosition, int ScalarTarget)
{
	return PhysicsField_SamplePhysicsScalarField(WorldPosition, ScalarTarget, PhysicsFieldTargets, PhysicsFieldTargetCount, PhysicsFieldClipmapCenter, PhysicsFieldClipmapDistance,
				PhysicsFieldClipmapExponent, PhysicsFieldClipmapCount, PhysicsFieldClipmapResolution, PhysicsFieldClipmapBuffer);
}

int MatPhysicsField_SamplePhysicsIntegerField(float3 WorldPosition, int IntegerTarget)
{
	return PhysicsField_SamplePhysicsIntegerField(WorldPosition, IntegerTarget, PhysicsFieldTargets, PhysicsFieldTargetCount, PhysicsFieldClipmapCenter, PhysicsFieldClipmapDistance,
				PhysicsFieldClipmapExponent, PhysicsFieldClipmapCount, PhysicsFieldClipmapResolution, PhysicsFieldClipmapBuffer);
	
}

float3 MatPhysicsField_EvalPhysicsVectorField(float3 WorldPosition, int VectorTarget)
{
	return PhysicsField_EvalPhysicsVectorField(WorldPosition, PhysicsFieldTargetMappings[VectorTarget].x, PhysicsFieldNodesParams, PhysicsFieldNodesOffsets, PhysicsFieldTargetsOffsets, PhysicsFieldTimeSeconds);
}

float MatPhysicsField_EvalPhysicsScalarField(float3 WorldPosition, int ScalarTarget)
{
	return PhysicsField_EvalPhysicsScalarField(WorldPosition, PhysicsFieldTargetMappings[ScalarTarget].y, PhysicsFieldNodesParams, PhysicsFieldNodesOffsets, PhysicsFieldTargetsOffsets, PhysicsFieldTimeSeconds);
}

int MatPhysicsField_EvalPhysicsIntegerField(float3 WorldPosition, int IntegerTarget)
{
	return PhysicsField_EvalPhysicsIntegerField(WorldPosition, PhysicsFieldTargetMappings[IntegerTarget].z, PhysicsFieldNodesParams, PhysicsFieldNodesOffsets, PhysicsFieldTargetsOffsets, PhysicsFieldTimeSeconds);
}

#endif // FEATURE_LEVEL >= FEATURE_LEVEL_SM5
