// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PhysicsFieldShared.ush
=============================================================================*/

#pragma once

#include "Common.ush"

#define MAX_PHYSICS_FIELD_TARGETS 32

float3 PhysicsField_ComputeSampleUV(in float3 LocalPosition, in int ClipmapIndex, in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount)
{
	const float ClipmapExtent = 2.0 * ClipmapDistance * pow((float) ClipmapExponent, (float) (ClipmapIndex + 1 - ClipmapCount));
	return LocalPosition / ClipmapExtent + 0.5;
}

int PhysicsField_GetClipmapIndex(in float3 LocalPosition, in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount)
{
	const int3 ClipmapIndices = floor(log(abs(LocalPosition) / ClipmapDistance) / log((float) (ClipmapExponent)) + int3(ClipmapCount, ClipmapCount, ClipmapCount));
	const int ClipmapIndex = max(max(ClipmapIndices[0], ClipmapIndices[1]), ClipmapIndices[2]);
	
	return clamp(ClipmapIndex, 0, ClipmapCount - 1);
}

void PhysicsField_ComputeVoxelIndices(in int3 PrevCell, in int ClipmapResolution, 
				out int I000, out int I100, out int I010, out int I110, 
				out int I001, out int I101, out int I011, out int I111)
{
	const int3 NextCell = PrevCell + int3(1, 1, 1);
	const int ResolutionSquare = ClipmapResolution * ClipmapResolution;
	
	const int PrevOffsetX = PrevCell.x;
	const int NextOffsetX = NextCell.x;
	
	const int PrevOffsetY = ClipmapResolution * PrevCell.y;
	const int NextOffsetY = ClipmapResolution * NextCell.y;
	
	const int PrevOffsetZ = ResolutionSquare * PrevCell.z;
	const int NextOffsetZ = ResolutionSquare * NextCell.z;
			
	I000 = (PrevOffsetX + PrevOffsetY + PrevOffsetZ);
	I100 = (NextOffsetX + PrevOffsetY + PrevOffsetZ);
	I010 = (PrevOffsetX + NextOffsetY + PrevOffsetZ);
	I110 = (NextOffsetX + NextOffsetY + PrevOffsetZ);
	I001 = (PrevOffsetX + PrevOffsetY + NextOffsetZ);
	I101 = (NextOffsetX + PrevOffsetY + NextOffsetZ);
	I011 = (PrevOffsetX + NextOffsetY + NextOffsetZ);
	I111 = (NextOffsetX + NextOffsetY + NextOffsetZ);
}

float3 PhysicsField_InterpolateVector(in int3 SampleIndex, in float3 SampleFraction, in int ClipmapResolution,  in Buffer<float> ClipmapBuffer, in int ClipmapCount)
{
	int I000 = 0, I100 = 0, I010 = 0, I110 = 0, I001 = 0, I101 = 0, I011 = 0, I111 = 0;
	PhysicsField_ComputeVoxelIndices(SampleIndex, ClipmapResolution, I000, I100, I010, I110, I001, I101, I011, I111);
	
	const int AttributeOffsetY = ClipmapResolution * ClipmapResolution * ClipmapResolution * ClipmapCount;
	const int AttributeOffsetZ = 2 * AttributeOffsetY;
	
	// Fetch corners
	const float3 V000 = float3(ClipmapBuffer[I000], ClipmapBuffer[I000 + AttributeOffsetY], ClipmapBuffer[I000 + AttributeOffsetZ]);
	const float3 V100 = float3(ClipmapBuffer[I100], ClipmapBuffer[I100 + AttributeOffsetY], ClipmapBuffer[I100 + AttributeOffsetZ]);
	const float3 V010 = float3(ClipmapBuffer[I010], ClipmapBuffer[I010 + AttributeOffsetY], ClipmapBuffer[I010 + AttributeOffsetZ]);
	const float3 V110 = float3(ClipmapBuffer[I110], ClipmapBuffer[I110 + AttributeOffsetY], ClipmapBuffer[I110 + AttributeOffsetZ]);
	const float3 V001 = float3(ClipmapBuffer[I001], ClipmapBuffer[I001 + AttributeOffsetY], ClipmapBuffer[I001 + AttributeOffsetZ]);
	const float3 V101 = float3(ClipmapBuffer[I101], ClipmapBuffer[I101 + AttributeOffsetY], ClipmapBuffer[I101 + AttributeOffsetZ]);
	const float3 V011 = float3(ClipmapBuffer[I011], ClipmapBuffer[I011 + AttributeOffsetY], ClipmapBuffer[I011 + AttributeOffsetZ]);
	const float3 V111 = float3(ClipmapBuffer[I111], ClipmapBuffer[I111 + AttributeOffsetY], ClipmapBuffer[I111 + AttributeOffsetZ]);

	// Blend x-axis
	const float3 V00 = lerp(V000, V100, SampleFraction.x);
	const float3 V01 = lerp(V001, V101, SampleFraction.x);
	const float3 V10 = lerp(V010, V110, SampleFraction.x);
	const float3 V11 = lerp(V011, V111, SampleFraction.x);

	// Blend y-axis
	const float3 V0 = lerp(V00, V10, SampleFraction.y);
	const float3 V1 = lerp(V01, V11, SampleFraction.y);

	// Blend z-axis
	return lerp(V0, V1, SampleFraction.z);
}

float PhysicsField_InterpolateScalar(in int3 SampleIndex, in float3 SampleFraction, in int ClipmapResolution,  in Buffer<float> ClipmapBuffer)
{
	int I000 = 0, I100 = 0, I010 = 0, I110 = 0, I001 = 0, I101 = 0, I011 = 0, I111 = 0;
	PhysicsField_ComputeVoxelIndices(SampleIndex, ClipmapResolution, I000, I100, I010, I110, I001, I101, I011, I111);
	
	// Fetch corners
	const float V000 = ClipmapBuffer[I000];
	const float V100 = ClipmapBuffer[I100];
	const float V010 = ClipmapBuffer[I010];
	const float V110 = ClipmapBuffer[I110];
	const float V001 = ClipmapBuffer[I001];
	const float V101 = ClipmapBuffer[I101];
	const float V011 = ClipmapBuffer[I011];
	const float V111 = ClipmapBuffer[I111];

	// Blend x-axis
	const float V00 = lerp(V000, V100, SampleFraction.x);
	const float V01 = lerp(V001, V101, SampleFraction.x);
	const float V10 = lerp(V010, V110, SampleFraction.x);
	const float V11 = lerp(V011, V111, SampleFraction.x);

	// Blend y-axis
	const float V0 = lerp(V00, V10, SampleFraction.y);
	const float V1 = lerp(V01, V11, SampleFraction.y);

	// Blend z-axis
	return lerp(V0, V1, SampleFraction.z);
}

void PhysicsField_GetSamplingInfos(in float3 LocalPosition, in int TargetIndex, 
			in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount, in int ClipmapResolution, out int3 SampleIndex, out float3 SampleFraction)
{
	const int ClipmapIndex = PhysicsField_GetClipmapIndex(LocalPosition, ClipmapDistance, ClipmapExponent, ClipmapCount);
	const float3 SampleUV = PhysicsField_ComputeSampleUV(LocalPosition, ClipmapIndex, ClipmapDistance, ClipmapExponent, ClipmapCount);
			
	float3 SampleVoxel = SampleUV * float3(ClipmapResolution - 1, ClipmapResolution - 1, ClipmapResolution - 1);
	SampleVoxel.z += (ClipmapCount * TargetIndex + ClipmapIndex) * ClipmapResolution;
			
	SampleIndex = floor(SampleVoxel);
	SampleFraction = SampleVoxel - SampleIndex;
}

float3 PhysicsField_SamplePhysicsVectorField(in float3 WorldPosition, in int VectorTarget, in int4 VectorTargets[MAX_PHYSICS_FIELD_TARGETS], in int TargetCount, in float3 ClipmapCenter,
			in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount, in int ClipmapResolution, in Buffer<float> ClipmapBuffer)
{
	const float3 LocalPosition = WorldPosition - ClipmapCenter;
	
	float3 FieldValue = float3(0, 0, 0);
	if (VectorTarget != -1 && VectorTarget < MAX_PHYSICS_FIELD_TARGETS && all(abs(LocalPosition) < float3(ClipmapDistance, ClipmapDistance, ClipmapDistance)))
	{
		const int TargetIndex = VectorTargets[VectorTarget].x;
		if (TargetIndex != -1)
		{
			int3 SampleIndex = int3(0, 0, 0);
			float3 SampleFraction = float3(0, 0, 0);
			PhysicsField_GetSamplingInfos(LocalPosition, TargetIndex, ClipmapDistance, ClipmapExponent, ClipmapCount, ClipmapResolution, SampleIndex, SampleFraction);
			
			FieldValue = PhysicsField_InterpolateVector(SampleIndex, SampleFraction, ClipmapResolution, ClipmapBuffer, ClipmapCount);
		}
	}
	return FieldValue;
}

float PhysicsField_SamplePhysicsScalarField(in float3 WorldPosition, in int ScalarTarget, in int4 ScalarTargets[MAX_PHYSICS_FIELD_TARGETS], in int TargetCount, in float3 ClipmapCenter,
			in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount, in int ClipmapResolution, in Buffer<float> ClipmapBuffer)
{
	const float3 LocalPosition = WorldPosition - ClipmapCenter;
	
	float FieldValue = 0.0;
	if (ScalarTarget != -1 && ScalarTarget < MAX_PHYSICS_FIELD_TARGETS && all(abs(LocalPosition) < float3(ClipmapDistance, ClipmapDistance, ClipmapDistance)))
	{
		const int TargetIndex = ScalarTargets[ScalarTarget].y;
		if (TargetIndex != -1)
		{
			int3 SampleIndex = int3(0, 0, 0);
			float3 SampleFraction = float3(0, 0, 0);
			PhysicsField_GetSamplingInfos(LocalPosition, TargetIndex, ClipmapDistance, ClipmapExponent, ClipmapCount, ClipmapResolution, SampleIndex, SampleFraction);
		
			FieldValue = PhysicsField_InterpolateScalar(SampleIndex, SampleFraction, ClipmapResolution, ClipmapBuffer);
		}
	}
	return FieldValue;
}

int PhysicsField_SamplePhysicsIntegerField(in float3 WorldPosition, in int IntegerTarget, in int4 IntegerTargets[MAX_PHYSICS_FIELD_TARGETS], in int TargetCount, in float3 ClipmapCenter,
			in float ClipmapDistance, in int ClipmapExponent, in int ClipmapCount, in int ClipmapResolution, in Buffer<float> ClipmapBuffer)
{
	const float3 LocalPosition = WorldPosition - ClipmapCenter;
	
	int FieldValue = 0.0;
	if (IntegerTarget != -1 && IntegerTarget < MAX_PHYSICS_FIELD_TARGETS && all(abs(LocalPosition) < float3(ClipmapDistance, ClipmapDistance, ClipmapDistance)))
	{
		const int TargetIndex = IntegerTargets[IntegerTarget].z;
		if (TargetIndex != -1)
		{
			int3 SampleIndex = int3(0, 0, 0);
			float3 SampleFraction = float3(0, 0, 0);
			PhysicsField_GetSamplingInfos(LocalPosition, TargetIndex, ClipmapDistance, ClipmapExponent, ClipmapCount, ClipmapResolution, SampleIndex, SampleFraction);
		
			FieldValue = PhysicsField_InterpolateScalar(SampleIndex, SampleFraction, ClipmapResolution, ClipmapBuffer);
		}
	}
	return FieldValue;
}

