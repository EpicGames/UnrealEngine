// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "CommonViewUniformBuffer.ush"
#include "SceneTextureParameters.ush"
#include "DeferredShadingCommon.ush"
#include "ShaderPrint.ush" 
#include "PositionReconstructionCommon.ush" 
 
#include "PhysicsFieldShared.ush"
#include "PhysicsFieldEval.ush"

RWTexture2D<float4> OutputTexture;
float2 OutputResolution;

Buffer<float4> BoundsMin;
Buffer<float4> BoundsMax;

uint BoundsOffset;
uint BoundsSize;

uint PrintOffset;

uint LocalTarget;
uint GlobalTarget;

#define MAX_STEP 10
#define FIELD_OPACITY 0.3

// Magma color palette
float4 GetFieldColor(in float AlphaValue) 
{
	const float t = AlphaValue;
	const float3 c0 = float3(-0.002136485053939582, -0.000749655052795221, -0.005386127855323933);
    const float3 c1 = float3(0.2516605407371642, 0.6775232436837668, 2.494026599312351);
    const float3 c2 = float3(8.353717279216625, -3.577719514958484, 0.3144679030132573);
    const float3 c3 = float3(-27.66873308576866, 14.26473078096533, -13.64921318813922);
    const float3 c4 = float3(52.17613981234068, -27.94360607168351, 12.94416944238394);
    const float3 c5 = float3(-50.76852536473588, 29.04658282127291, 4.23415299384598);
    const float3 c6 = float3(18.65570506591883, -11.48977351997711, -5.601961508734096);

    return float4(c0+t*(c1+t*(c2+t*(c3+t*(c4+t*(c5+t*c6))))),1.0);
}

void FillRayInfos(in uint2 ScreenCoord, inout float3 RayBegin, inout float3 RayEnd, inout float3 RayDirection)
{
	const float2 PixelCoord = ScreenCoord + 0.5f;
	const float2 UV = PixelCoord / float2(OutputResolution);
	const float ClosestDepth = ConvertFromDeviceZ(0.0000001f);//ConvertFromDeviceZ(SceneDepthTexture.Load(uint3(PixelCoord, 0)));

	const float3 ViewOffset = DFDemote(PrimaryView.PreViewTranslation);
	RayBegin = PrimaryView.TranslatedWorldCameraOrigin - ViewOffset;
	RayEnd = ReconstructTranslatedWorldPositionFromDepth(UV, ClosestDepth) - ViewOffset;
	RayDirection = (RayEnd - RayBegin);
}

float ComputeStepSize(in float2 HitBounds, in float3 RayDirection, in int BoundsIndex, inout float DeltaStep)
{
	DeltaStep = (HitBounds.y-HitBounds.x)/float(MAX_STEP-1);
	return DeltaStep*length(RayDirection)/(0.5*length(BoundsMax[BoundsIndex].xyz - BoundsMin[BoundsIndex].xyz));
}

void SetOutputTexture(inout float RayDensity, in int BoundsIndex, in uint2 PixelCoord)
{
	RayDensity /= (BoundsMax[BoundsIndex].w-BoundsMin[BoundsIndex].w);
	const float4 DensityColor = GetFieldColor(min(1.0,abs(RayDensity)));

	const float4 FieldColor = (RayDensity < 0.0 ) ? float4(DensityColor.y, DensityColor.x, DensityColor.z, 1.0) : float4(DensityColor.z, DensityColor.x, DensityColor.y, 1.0);
	OutputTexture[PixelCoord] = FIELD_OPACITY * OutputTexture[PixelCoord] + (1.0-FIELD_OPACITY) * FieldColor;
}

// Vector field visualisation
#if PERMUTATION_FIELD == 0

FFontColor GetVectorTargetColor()
{
	if(GlobalTarget == LINEAR_FORCE)
	{
		return FontRed;
	}
	else if(GlobalTarget == LINEAR_VELOCITY)
	{
		return FontGreen;
	}
	else if(GlobalTarget == ANGULAR_TORQUE)
	{
		return FontBlue;
	}
	else if(GlobalTarget == ANGULAR_VELOCITY)
	{
		return FontYellow;
	}
	return FontWhite;
}

void PrintVectorField()
{
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50.0f, 200.0f));
	Context.Pos.y += Context.Config.FontSpacing.y * PrintOffset;

	const FFontColor VectorColor = GetVectorTargetColor();

	Print(Context, TEXT("Vector Field | Type = "));
	if(GlobalTarget == LINEAR_FORCE)
	{
		Print(Context, TEXT("Linear Force "), VectorColor);
	}
	else if(GlobalTarget == LINEAR_VELOCITY)
	{
		Print(Context, TEXT("Linear Velocity "), VectorColor);
	}
	else if(GlobalTarget == ANGULAR_TORQUE)
	{
		Print(Context, TEXT("Angular Torque "), VectorColor);
	}
	else if(GlobalTarget == ANGULAR_VELOCITY)
	{
		Print(Context, TEXT("Angular Velocity "), VectorColor);
	}

	Print(Context, TEXT("| Count = "));
	Print(Context, BoundsSize);
}

[numthreads(4, 4, 4)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID)
{
	const FFontColor AABBColor = GetVectorTargetColor();
	if(DispatchThreadId.x < VOXEL_RESOLUTION * BoundsSize && DispatchThreadId.y < VOXEL_RESOLUTION && DispatchThreadId.z < VOXEL_RESOLUTION)
	{
		const uint BoundsIndex = BoundsOffset + DispatchThreadId.x / VOXEL_RESOLUTION;
		const uint3 VoxelIndex = uint3(DispatchThreadId.x % VOXEL_RESOLUTION, DispatchThreadId.y, DispatchThreadId.z);

		const float3 BoundsExtent = (BoundsMax[BoundsIndex].xyz - BoundsMin[BoundsIndex].xyz);

		const float3 BoundLength = BoundsExtent / VOXEL_RESOLUTION;
		const float VoxelLength = max(max(BoundLength.x, BoundLength.y), BoundLength.z);

		const uint3 BoundResolution = uint3( BoundsExtent.x / VoxelLength, BoundsExtent.y / VoxelLength, BoundsExtent.z / VoxelLength);

		if( VoxelIndex.x < BoundResolution.x && VoxelIndex.y < BoundResolution.y && VoxelIndex.z < BoundResolution.z )
		{
			const float3 VoxelCenter = BoundsMin[BoundsIndex].xyz + (VoxelIndex + float3(0.5,0.5,0.5)) * VoxelLength;

#if PERMUTATION_EVAL == 1
			float3 FieldVector = PhysicsField_SamplePhysicsVectorField(VoxelCenter, LocalTarget, View.PhysicsFieldTargets, View.PhysicsFieldTargetCount, View.PhysicsFieldClipmapCenter, View.PhysicsFieldClipmapDistance,
				View.PhysicsFieldClipmapExponent, View.PhysicsFieldClipmapCount, View.PhysicsFieldClipmapResolution, View.PhysicsFieldClipmapBuffer);
#endif

#if PERMUTATION_EVAL == 0
			float3 FieldVector = PhysicsField_EvalPhysicsVectorField(VoxelCenter, GlobalTarget, View.PhysicsFieldNodesParams, View.PhysicsFieldNodesOffsets, View.PhysicsFieldTargetsOffsets, View.GameTime);
#endif

			const float VectorLength = length(FieldVector);
			if(VectorLength > 0) 
			{
				FieldVector /= VectorLength;
				const float4 FieldColor = float4((FieldVector + float3(1,1,1))/2.0, min(1.0, VectorLength));
				AddLineWS(VoxelCenter, VoxelCenter + FieldVector * VoxelLength , FieldColor, FieldColor );
			}
		}
		if(DispatchThreadId.x == 0 && DispatchThreadId.y == 0 && DispatchThreadId.z == 0) 
		{
			for(int BoundsIndex = BoundsOffset, BoundsEnd = BoundsOffset + BoundsSize; BoundsIndex < BoundsEnd; ++BoundsIndex)
			{
				AddAABBWS(BoundsMin[BoundsIndex].xyz, BoundsMax[BoundsIndex].xyz, float4(AABBColor.Color.x, AABBColor.Color.y, AABBColor.Color.z, 1));
			}
			PrintVectorField();
		}
	}
}

#endif

#if PERMUTATION_FIELD == 1 || PERMUTATION_FIELD == 2

FFontColor GetScalarTargetColor()
{
	if(GlobalTarget == EXTERNAL_STRAIN)
	{
		return FontCyan;
	}
	else if(GlobalTarget == INTERNAL_STRAIN)
	{
		return FontMagenta;
	}
	else if(GlobalTarget == DISABLED_THRESHOLD)
	{
		return FontOrange;
	}
	else if(GlobalTarget == SLEEPING_THRESHOLD)
	{
		return FontPurple;
	}
	else if(GlobalTarget == KILL_PARTICLES)
	{
		return FontTurquoise;
	}
	return FontWhite;
}

void PrintScalarField()
{
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50.0f, 200.0f));
	Context.Pos.y += Context.Config.FontSpacing.y * PrintOffset;

	const FFontColor ScalarColor = GetScalarTargetColor();

	Print(Context, TEXT("Scalar Field | Type = "));
	if(GlobalTarget == EXTERNAL_STRAIN)
	{
		Print(Context, TEXT("External Strain "), ScalarColor);
	}
	else if(GlobalTarget == INTERNAL_STRAIN)
	{
		Print(Context, TEXT("Internal Strain "), ScalarColor);
	}
	else if(GlobalTarget == DISABLED_THRESHOLD)
	{
		Print(Context, TEXT("Disabled Threshold "), ScalarColor);
	}
	else if(GlobalTarget == SLEEPING_THRESHOLD)
	{
		Print(Context, TEXT("Sleeping Threshold "), ScalarColor);
	}
	else if(GlobalTarget == KILL_PARTICLES)
	{
		Print(Context, TEXT("Kill Particles "), ScalarColor);
	}

	Print(Context, TEXT("| Count = "));
	Print(Context, BoundsSize);
}

FFontColor GetIntegerTargetColor()
{
	if(GlobalTarget == DYNAMIC_STATE)
	{
		return FontSilver;
	}
	else if(GlobalTarget == COLLISION_GROUP)
	{
		return FontEmerald;
	}
	else if(GlobalTarget == ACTIVATE_DISABLED)
	{
		return FontWhite;
	}
	return FontWhite;
}

void PrintIntegerField()
{
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50.0f, 200.0f));
	Context.Pos.y += Context.Config.FontSpacing.y * PrintOffset;

	const FFontColor IntegerColor = GetIntegerTargetColor();

	Print(Context, TEXT("Integer Field | Type = "));
	if(GlobalTarget == DYNAMIC_STATE)
	{
		Print(Context, TEXT("Dynamic State "), IntegerColor);
	}
	else if(GlobalTarget == COLLISION_GROUP)
	{
		Print(Context, TEXT("Collision Group "), IntegerColor);
	}
	else if(GlobalTarget == ACTIVATE_DISABLED)
	{
		Print(Context, TEXT("Activate Disabled "), IntegerColor);
	}

	Print(Context, TEXT("| Count = "));
	Print(Context, BoundsSize);
}



// Integer and scalar field visualisation
[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID)
{
	#if PERMUTATION_FIELD == 2
	const FFontColor AABBColor = GetIntegerTargetColor();
	#endif

	#if PERMUTATION_FIELD == 1
	const FFontColor AABBColor = GetScalarTargetColor();
	#endif

	if(DispatchThreadId.x < OutputResolution.x && DispatchThreadId.y < OutputResolution.y)
	{
		float3 RayBegin, RayEnd, RayDirection;
		FillRayInfos(DispatchThreadId.xy, RayBegin, RayEnd, RayDirection);

		for(int BoundsIndex = BoundsOffset, BoundsEnd = BoundsOffset + BoundsSize; BoundsIndex < BoundsEnd; ++BoundsIndex)
		{
			const float2 HitBounds = LineBoxIntersect(RayBegin, RayEnd, BoundsMin[BoundsIndex].xyz, BoundsMax[BoundsIndex].xyz);
			if (HitBounds.x < HitBounds.y)
			{
				float DeltaStep = 0.0;
				const float StepSize = ComputeStepSize(HitBounds, RayDirection, BoundsIndex, DeltaStep);

				float RayDensity = 0;
				for (uint StepIndex = 0; StepIndex < MAX_STEP; ++StepIndex)
				{			
					const float3 WorldPosition = RayBegin + (StepIndex * DeltaStep + HitBounds.x) * RayDirection;

#if PERMUTATION_FIELD == 2

#if PERMUTATION_EVAL == 1
					const int FieldInteger = PhysicsField_SamplePhysicsIntegerField(WorldPosition, LocalTarget, View.PhysicsFieldTargets, View.PhysicsFieldTargetCount, View.PhysicsFieldClipmapCenter, View.PhysicsFieldClipmapDistance,
						View.PhysicsFieldClipmapExponent, View.PhysicsFieldClipmapCount, View.PhysicsFieldClipmapResolution, View.PhysicsFieldClipmapBuffer);
#endif

#if PERMUTATION_EVAL == 0
					const int FieldInteger = PhysicsField_EvalPhysicsIntegerField(WorldPosition, GlobalTarget, View.PhysicsFieldNodesParams, View.PhysicsFieldNodesOffsets, View.PhysicsFieldTargetsOffsets, View.GameTime);
#endif
					RayDensity += FieldInteger * StepSize;
#endif

#if PERMUTATION_FIELD == 1

#if PERMUTATION_EVAL == 1
					const float FieldScalar = PhysicsField_SamplePhysicsScalarField(WorldPosition, LocalTarget, View.PhysicsFieldTargets, View.PhysicsFieldTargetCount, View.PhysicsFieldClipmapCenter, View.PhysicsFieldClipmapDistance,
						View.PhysicsFieldClipmapExponent, View.PhysicsFieldClipmapCount, View.PhysicsFieldClipmapResolution, View.PhysicsFieldClipmapBuffer);
#endif

#if PERMUTATION_EVAL == 0
					float FieldScalar = PhysicsField_EvalPhysicsScalarField(WorldPosition, GlobalTarget, View.PhysicsFieldNodesParams, View.PhysicsFieldNodesOffsets, View.PhysicsFieldTargetsOffsets, View.GameTime);
#endif
					
					RayDensity += FieldScalar * StepSize;
#endif
				}
				SetOutputTexture(RayDensity, BoundsIndex, DispatchThreadId.xy);
			}
		}
		if(DispatchThreadId.x == 0 && DispatchThreadId.y == 0) 
		{
			for(int BoundsIndex = BoundsOffset, BoundsEnd = BoundsOffset + BoundsSize; BoundsIndex < BoundsEnd; ++BoundsIndex)
			{
				AddAABBWS(BoundsMin[BoundsIndex].xyz, BoundsMax[BoundsIndex].xyz, 
					float4(AABBColor.Color.x, AABBColor.Color.y, AABBColor.Color.z, 1));
			}
#if PERMUTATION_FIELD == 2
			PrintIntegerField();
#endif

#if PERMUTATION_FIELD == 1
			PrintScalarField();
#endif
		}
	}
}

#endif

#if PERMUTATION_FIELD == 3

// Reset field visualisation
[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID)
{
	if(DispatchThreadId.x < OutputResolution.x && DispatchThreadId.y < OutputResolution.y)
	{
		float3 RayBegin, RayEnd, RayDirection;
		FillRayInfos(DispatchThreadId.xy, RayBegin, RayEnd, RayDirection);

		for(int BoundsIndex = BoundsOffset, BoundsEnd = BoundsOffset + BoundsSize; BoundsIndex < BoundsEnd; ++BoundsIndex)
		{
			const float2 HitBounds = LineBoxIntersect(RayBegin, RayEnd, BoundsMin[BoundsIndex].xyz, BoundsMax[BoundsIndex].xyz);
			if (HitBounds.x < HitBounds.y)
			{
				OutputTexture[DispatchThreadId.xy] = FIELD_OPACITY * OutputTexture[DispatchThreadId.xy] + (1.0-FIELD_OPACITY) * float4(0,0,0,1);
			}
		}
	}
}

#endif
