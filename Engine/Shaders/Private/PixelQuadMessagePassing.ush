// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PixelQuadMessagePassing.usf: Contains functions to do pixel quad message passing within the pixel shaders.
=========================================================================*/

#pragma once


// ddx/y_fine() are not supported in SM4.
#define HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT ((FEATURE_LEVEL >= FEATURE_LEVEL_SM5) && !MOBILE_EMULATION && PIXELSHADER)


// Pixel quad message passing in SM5 requires some thread informations.
struct FPQMPContext
{
	// Position of the pixel within the 2x2 pixel quad.
	float2 PixelPos;

	// Signed normalized  2D vector for each pixel in a 2x2 quad from the centre. Will be either +1 or -1 for each axis
	float2  QuadVector;
};


// Returns a required thread context to perform pixel quad message passing in SM5.
FPQMPContext PQMPInit(float2 SvPosition)
{
	FPQMPContext PQMP;
#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
	PQMP.PixelPos   = floor(2.0 * frac(SvPosition * 0.5));
	uint2 ScreenPos = uint2(SvPosition);
	PQMP.QuadVector = float2( float(ScreenPos.x & 1) * 2.0 - 1.0, float(ScreenPos.y & 1) * 2.0 - 1.0);
#else
	PQMP.PixelPos	= float2(0, 0);
	PQMP.QuadVector = float2(0, 0);
#endif
	return PQMP;
}


// Get the average value of <v> across the pixel quad.
float PQMPAverage(FPQMPContext PQMP, float v)
{
#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
	v    = v + (0.5 - PQMP.PixelPos.x) * ddx_fine(v);
	return v + (0.5 - PQMP.PixelPos.y) * ddy_fine(v);
#else
	return v;
#endif
}

float2 PQMPAverage(FPQMPContext PQMP, float2 v)
{
#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
	v    = v + (0.5 - PQMP.PixelPos.x) * ddx_fine(v);
	return v + (0.5 - PQMP.PixelPos.y) * ddy_fine(v);
#else
	return v;
#endif
}


// Get the value of the horizontal neighbour in the same pixel quad
float PQMPReadX(FPQMPContext PQMP, float v)
{
#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
	#if COMPILER_SUPPORTS_QUAD_PASS
		return QuadReadAcrossX(v);
	#else
		float dX = ddx_fine(v);
		return v - (dX * PQMP.QuadVector.x);
	#endif
#else
	return v;
#endif
}

// Get the value of the vertical neighbour in the same pixel quad
float PQMPReadY(FPQMPContext PQMP, float v)
{
#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
	#if COMPILER_SUPPORTS_QUAD_PASS
		return QuadReadAcrossY(v);
	#else
		float dY = ddy_fine(v);
		return v - (dY * PQMP.QuadVector.y);
	#endif
#else
	return v;
#endif
}

float2 PQMPReadX(FPQMPContext PQMP, float2 v)
{
	return float2( PQMPReadX(PQMP, v.x), PQMPReadX(PQMP, v.y));
}

float2 PQMPReadY(FPQMPContext PQMP, float2 v)
{
	return float2( PQMPReadY(PQMP, v.x), PQMPReadY(PQMP, v.y));
}

float3 PQMPReadX(FPQMPContext PQMP, float3 v)
{
	return float3( PQMPReadX(PQMP, v.x), PQMPReadX(PQMP, v.y), PQMPReadX(PQMP, v.z));
}

float3 PQMPReadY(FPQMPContext PQMP, float3 v)
{
	return float3( PQMPReadY(PQMP, v.x), PQMPReadY(PQMP, v.y), PQMPReadY(PQMP, v.z));
}


