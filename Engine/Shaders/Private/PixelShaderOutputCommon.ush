// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PixelShaderOutputCommon.ush: To allow PS input/output passed into functions
through a single struct, allowing for a more readable code
(less #ifdefs, reducing the boolean hell)
=============================================================================*/

#include "ShaderOutputCommon.ush"

PIXELSHADER_EARLYDEPTHSTENCIL
void MainPS
	(
#if PIXELSHADEROUTPUT_INTERPOLANTS || PIXELSHADEROUTPUT_BASEPASS
#if IS_NANITE_PASS && !IS_NANITE_TRANSLUCENCY
		FNaniteFullscreenVSToPS NaniteInterpolants,
#else
		FVertexFactoryInterpolantsVSToPS Interpolants,
#endif
#endif
#if PIXELSHADEROUTPUT_BASEPASS
		FBasePassInterpolantsVSToPS BasePassInterpolants,
#elif PIXELSHADEROUTPUT_MESHDECALPASS
		FMeshDecalInterpolants MeshDecalInterpolants,
#endif
	#if STORE_CLIP_ZW_IN_INTERPOLATOR
		in INPUT_POSITION_QUALIFIERS float4 SvPosition_XY : SV_Position,		// after all interpolators
	#else
		in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position,			// after all interpolators
	#endif
		in FStereoPSInput StereoInput
		OPTIONAL_IsFrontFace

#if PIXELSHADEROUTPUT_MRT0
	#if DUAL_SOURCE_COLOR_BLENDING_ENABLED && MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING
		, out float4 OutTarget0 DUAL_SOURCE_BLENDING_SLOT(0) : SV_Target0
	#else
		, out float4 OutTarget0 : SV_Target0
	#endif
#endif

#if PIXELSHADEROUTPUT_MRT1
	#if DUAL_SOURCE_COLOR_BLENDING_ENABLED && MATERIAL_WORKS_WITH_DUAL_SOURCE_COLOR_BLENDING
		, out float4 OutTarget1 DUAL_SOURCE_BLENDING_SLOT(1) : SV_Target1
	#else
		, out float4 OutTarget1 : SV_Target1
	#endif
#endif

#if PIXELSHADEROUTPUT_MRT2
		, out float4 OutTarget2 : SV_Target2
#endif

#if PIXELSHADEROUTPUT_MRT3
		, out float4 OutTarget3 : SV_Target3
#endif

#define SUBSTRATE_OUTPUT_GBUFFER  (SUBSTRATE_OPAQUE_DEFERRED && !MATERIAL_IS_SKY && SUBSTRATE_GBUFFER_FORMAT==1)

#if SUBSTRATE_OUTPUT_GBUFFER
#if SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT != 3
#error Substrate SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT has been updated but not the uint MRTs
#endif
#if SUBSTRATE_FIRST_MRT_INDEX > 4
#error Substrate cannot map to such a case
#endif
#if SUBSTRATE_FIRST_MRT_INDEX == 4
		, out uint SubstrateOutput4 : SV_Target4
		, out uint SubstrateOutput5 : SV_Target5
		, out uint SubstrateOutput6 : SV_Target6
		, out SUBSTRATE_TOP_LAYER_TYPE SubstrateOutput7 : SV_Target7
#elif SUBSTRATE_FIRST_MRT_INDEX == 3
		, out uint SubstrateOutput3 : SV_Target3
		, out uint SubstrateOutput4 : SV_Target4
		, out uint SubstrateOutput5 : SV_Target5
		, out SUBSTRATE_TOP_LAYER_TYPE SubstrateOutput6 : SV_Target6
#elif SUBSTRATE_FIRST_MRT_INDEX == 2
		, out uint SubstrateOutput2 : SV_Target2
		, out uint SubstrateOutput3 : SV_Target3
		, out uint SubstrateOutput4 : SV_Target4
		, out SUBSTRATE_TOP_LAYER_TYPE SubstrateOutput5 : SV_Target5
#else
		, out uint SubstrateOutput1 : SV_Target1
		, out uint SubstrateOutput2 : SV_Target2
		, out uint SubstrateOutput3 : SV_Target3
		, out SUBSTRATE_TOP_LAYER_TYPE SubstrateOutput4 : SV_Target4
#endif
#else // SUBSTRATE_OUTPUT_GBUFFER

#if PIXELSHADEROUTPUT_MRT4
		, out float4 OutTarget4 : SV_Target4
#endif

#if PIXELSHADEROUTPUT_MRT5
		, out float4 OutTarget5 : SV_Target5
#endif

#if PIXELSHADEROUTPUT_MRT6
		, out float4 OutTarget6 : SV_Target6
#endif

#if PIXELSHADEROUTPUT_MRT7
		, out float4 OutTarget7 : SV_Target7
#endif

#endif // SUBSTRATE_OUTPUT_GBUFFER

		// todo: if we are not inside a volumetric decal we could use OPTIONAL_OutDepthConservative for better performance, this would require a shader permutation
		OPTIONAL_OutDepthConservative

#if PIXELSHADEROUTPUT_COVERAGE || PIXELSHADEROUTPUT_A2C
#if PIXELSHADEROUTPUT_A2C
		, in uint InCoverage : SV_Coverage
#endif
		, out uint OutCoverage : SV_Coverage
#endif
	)
{
#if STORE_CLIP_ZW_IN_INTERPOLATOR
	const float4 SvPosition = float4(SvPosition_XY.x, SvPosition_XY.y, Interpolants.ClipZW.x, Interpolants.ClipZW.y);
#endif
	// ---------------------------------------------------------------------------------
#if IS_NANITE_PASS && !IS_NANITE_TRANSLUCENCY && (PIXELSHADEROUTPUT_INTERPOLANTS || PIXELSHADEROUTPUT_BASEPASS)
	FVertexFactoryInterpolantsVSToPS Interpolants = (FVertexFactoryInterpolantsVSToPS)0;
	Interpolants.ViewIndex = NaniteInterpolants.ViewIndex;
#if INSTANCED_STEREO
	// Nanite fullscreen VS is run for the whole side-by-side RT and the primary (left) view is View0 so figure out EyeIndex based on that - cannot do it in VS since quads can span views.
	// Revisit if we need to support > 1 instanced view or non side-by-side views
	Interpolants.EyeIndex = (SvPosition.x >= (View.ViewRectMin.x + View.ViewSizeAndInvSize.x)) ? 1 : 0;
#endif
#endif

	FPixelShaderIn PixelShaderIn = (FPixelShaderIn)0;
	FPixelShaderOut PixelShaderOut = (FPixelShaderOut)0;

#if PIXELSHADEROUTPUT_COVERAGE || PIXELSHADEROUTPUT_A2C
#if PIXELSHADEROUTPUT_A2C
	PixelShaderIn.Coverage = InCoverage;
#else
	PixelShaderIn.Coverage = 0xF;
#endif
	PixelShaderOut.Coverage = PixelShaderIn.Coverage;
#endif 

	PixelShaderIn.SvPosition = SvPosition;
	PixelShaderIn.bIsFrontFace = bIsFrontFace;

	StereoSetupPS(StereoInput);

#if PIXELSHADEROUTPUT_BASEPASS
	FPixelShaderInOut_MainPS(Interpolants, BasePassInterpolants, PixelShaderIn, PixelShaderOut, GetEyeIndex(StereoInput));
#elif PIXELSHADEROUTPUT_MESHDECALPASS
	FPixelShaderInOut_MainPS(Interpolants, MeshDecalInterpolants, PixelShaderIn, PixelShaderOut, GetEyeIndex(StereoInput));
#elif PIXELSHADEROUTPUT_INTERPOLANTS
	FPixelShaderInOut_MainPS(Interpolants, PixelShaderIn, PixelShaderOut);
#else
	FPixelShaderInOut_MainPS(PixelShaderIn, PixelShaderOut, GetEyeIndex(StereoInput));
#endif

#if PIXELSHADEROUTPUT_MRT0
	OutTarget0 = PixelShaderOut.MRT[0];
#endif

#if PIXELSHADEROUTPUT_MRT1
	OutTarget1 = PixelShaderOut.MRT[1];
#endif

#if PIXELSHADEROUTPUT_MRT2
	OutTarget2 = PixelShaderOut.MRT[2];
#endif

#if PIXELSHADEROUTPUT_MRT3
	OutTarget3 = PixelShaderOut.MRT[3];
#endif

#if SUBSTRATE_OUTPUT_GBUFFER

	// In this case, here is the gbuffer pattern
	// MRT0 is pixel color
	// MRT1 is velocity if enabled or precomputed shadow if velocity if disabled and precomputed shadow enabled
	// MRT2 is precomputed shadow if both velocity and prec shadow are enabled.
	// After, Substrate output are appended (3 uint output)

#if SUBSTRATE_FIRST_MRT_INDEX == 4
	SubstrateOutput4 = PixelShaderOut.SubstrateOutput[0];
	SubstrateOutput5 = PixelShaderOut.SubstrateOutput[1];
	SubstrateOutput6 = PixelShaderOut.SubstrateOutput[2];
	SubstrateOutput7 = PixelShaderOut.SubstrateTopLayerData;
#elif SUBSTRATE_FIRST_MRT_INDEX == 3
	SubstrateOutput3 = PixelShaderOut.SubstrateOutput[0];
	SubstrateOutput4 = PixelShaderOut.SubstrateOutput[1];
	SubstrateOutput5 = PixelShaderOut.SubstrateOutput[2];
	SubstrateOutput6 = PixelShaderOut.SubstrateTopLayerData;
#elif SUBSTRATE_FIRST_MRT_INDEX == 2
	SubstrateOutput2 = PixelShaderOut.SubstrateOutput[0];
	SubstrateOutput3 = PixelShaderOut.SubstrateOutput[1];
	SubstrateOutput4 = PixelShaderOut.SubstrateOutput[2];
	SubstrateOutput5 = PixelShaderOut.SubstrateTopLayerData;
#else
	SubstrateOutput1 = PixelShaderOut.SubstrateOutput[0];
	SubstrateOutput2 = PixelShaderOut.SubstrateOutput[1];
	SubstrateOutput3 = PixelShaderOut.SubstrateOutput[2];
	SubstrateOutput4 = PixelShaderOut.SubstrateTopLayerData;
#endif

#else // SUBSTRATE_OUTPUT_GBUFFER

#if PIXELSHADEROUTPUT_MRT4
	OutTarget4 = PixelShaderOut.MRT[4];
#endif

#if PIXELSHADEROUTPUT_MRT5
	OutTarget5 = PixelShaderOut.MRT[5];
#endif

#if PIXELSHADEROUTPUT_MRT6
	OutTarget6 = PixelShaderOut.MRT[6];
#endif

#if PIXELSHADEROUTPUT_MRT7
	OutTarget7 = PixelShaderOut.MRT[7];
#endif

#endif // SUBSTRATE_OUTPUT_GBUFFER

#if PIXELSHADEROUTPUT_COVERAGE || PIXELSHADEROUTPUT_A2C
	OutCoverage = PixelShaderOut.Coverage;
#endif 

#if OUTPUT_PIXEL_DEPTH_OFFSET
	OutDepth = PixelShaderOut.Depth;
#endif 
}
