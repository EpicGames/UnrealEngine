// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 PlanarReflectionShaders.usf
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "PlanarReflectionShared.ush"
#include "SceneTextureParameters.ush"
#include "Substrate/Substrate.ush"

Texture2D SceneColorInputTexture;
SamplerState SceneColorInputSampler;

float4 SampleSceneColorInput(float2 UV)
{
	return Texture2DSampleLevel(SceneColorInputTexture, SceneColorInputSampler, UV, 0);
}

float GaussianWeight1d(float u, float KernelRadius)
{
	float TwoKernelRadiusSq = 2 * KernelRadius * KernelRadius;
	return exp(-u * u / TwoKernelRadiusSq) / sqrt(PI * TwoKernelRadiusSq);
}

float InvPrefilterRoughnessDistance;

void VerticalSample(float2 InUV, float y, float KernelRadiusY, float CenterDistanceToPlane, float CenterReflectionDistance, bool bSearchUp, inout float4 AccumulatedColor, inout float AccumulatedWeight)
{
	float2 SampleUV = float2(InUV.x, InUV.y + y * View.BufferSizeAndInvSize.w);
	float SampleDepth = CalcSceneDepth(SampleUV);
	float SampleReflectionDistance = max(SampleDepth - CenterDistanceToPlane, 0);
	float ScatterKernelSize = max(KernelRadiusY * saturate(SampleReflectionDistance * InvPrefilterRoughnessDistance), 1);
	float SampleWeight = GaussianWeight1d(y, ScatterKernelSize);
	float ColorHalfTexelOffset = .5f * View.BufferSizeAndInvSize.w;
	float YPosition = InUV.y * View.BufferSizeAndInvSize.y + y;

	if (bSearchUp)
	{
		// Mask by depth, don't accept contribution from pixels much further than the mirror reflection depth
		// This prevents reading from background areas that were revealed due to the clip plane
		float DepthMask = SampleReflectionDistance < CenterReflectionDistance * 2;
		SampleWeight *= DepthMask;
		// Can't use bilinear filtering when we are depth masking
		ColorHalfTexelOffset = 0;

		float ViewportMask = YPosition > View.ViewRectMin.y + 0.5;
		SampleWeight *= ViewportMask;
	}
	else
	{
		float ViewportMask = YPosition < View.ViewRectMin.y + View.ViewSizeAndInvSize.y - 0.5;
		SampleWeight *= ViewportMask;
	}

	// Half texel offset for bilinear
	float4 SampleColor = SampleSceneColorInput(SampleUV + float2(0, ColorHalfTexelOffset));

	// SampleColor might contain NaNs when sampling out of the view rect, and multiplying by 0 won't help
	AccumulatedColor += -min(-SampleColor, 0.0) * SampleWeight;
	AccumulatedWeight += SampleWeight;
}

float KernelRadiusY;

void PrefilterPlanarReflectionPS(
	float2 InUV : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	float2 BufferUV = min(InUV, (View.ViewSizeAndInvSize.xy + View.ViewRectMin.xy - 0.5) * View.BufferSizeAndInvSize.zw);

#if ENABLE_PLANAR_REFLECTIONS_PREFILTER
	BRANCH
	if (KernelRadiusY > 0)
	{
		// Compute distance from camera origin to the camera vector intersection with the reflection plane
		float CenterDistanceToPlane = -(dot(PrimaryView.TranslatedWorldCameraOrigin, PlanarReflectionStruct.ReflectionPlane.xyz) - PlanarReflectionStruct.ReflectionPlane.w) / dot(normalize(ScreenVector), PlanarReflectionStruct.ReflectionPlane.xyz);
		// Note: reading from depth after the temporal AA pass
		// This works without jittering because we disable temporal AA jittering for planar reflections
		float CenterSceneDepth = CalcSceneDepth(BufferUV);
		float CenterReflectionDistance = max(CenterSceneDepth - CenterDistanceToPlane, 0);

		float4 AccumulatedColor = 0;
		float AccumulatedWeight = 0;

		// Two texels at a time with bilinear
		{
			LOOP
			for (float y = 0; y < KernelRadiusY; y += 2)
			{
				VerticalSample(BufferUV, y, KernelRadiusY, CenterDistanceToPlane, CenterReflectionDistance, false, AccumulatedColor, AccumulatedWeight);
			}
		}

		{
			LOOP
			for (float y = -2; y > -KernelRadiusY; y -= 2)
			{
				VerticalSample(BufferUV, y, KernelRadiusY, CenterDistanceToPlane, CenterReflectionDistance, true, AccumulatedColor, AccumulatedWeight);
			}
		}

		AccumulatedColor = AccumulatedColor / max(AccumulatedWeight, .0001f);

		// We cleared the planar reflection alpha to 1, and base pass writes 0
		float ValidContentMask = 1 - AccumulatedColor.a;
		OutColor = float4(AccumulatedColor.rgb, ValidContentMask);
	}
	else
#endif // ENABLE_PLANAR_REFLECTIONS_PREFILTER
	{
		float4 SceneColorValue = SampleSceneColorInput(BufferUV);

		#if (ES3_1_PROFILE)
			// when mesh draw command pipline is active mobile scene color contains depth in the alpha channel, any value > 1.0 has been written to.
			if( SceneColorValue.a > 1.0)
			{
				SceneColorValue.a = 0;
			}
		#endif

		// We cleared the planar reflection alpha to 1, and base pass writes 0
		float ValidContentMask = 1 - SceneColorValue.a;
		OutColor = float4(SceneColorValue.xyz, ValidContentMask);
	}
}

#if SHADING_PATH_DEFERRED

void PlanarReflectionPS(
	float2 BufferUV : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();
	OutColor = 0;


#if SUBSTRATE_GBUFFER_FORMAT==1

	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SVPos.xy, 0)));

	float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
	float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	float3 TranslatedWorldPosition = ResolvedView.TranslatedWorldCameraOrigin + ScreenVector * SceneDepth;

	const float Roughness = TopLayerData.Roughness;
	const float3 WorldNormal = TopLayerData.WorldNormal;
	const bool bNoMaterial = !IsSubstrateMaterial(TopLayerData);

	// Skip unlit pixels (skybox)
	BRANCH
	if (!bNoMaterial)
	{
		OutColor = ComputePlanarReflections(TranslatedWorldPosition, WorldNormal, Roughness);

		OutColor.xyz *= View.PreExposure;
	}

#else

	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);

	float3 TranslatedWorldPosition = GetTranslatedWorldCameraPosFromView(SVPos.xy) + ScreenVector * GBuffer.Depth;

	// Skip unlit pixels (skybox)
	BRANCH
	if (GBuffer.ShadingModelID != SHADINGMODELID_UNLIT)
	{
		OutColor = ComputePlanarReflections(TranslatedWorldPosition, GBuffer.WorldNormal, GBuffer.Roughness);

		OutColor.xyz *= View.PreExposure;
	}

#endif

}

#endif