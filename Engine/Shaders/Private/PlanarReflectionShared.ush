// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 PlanarReflectionShared.usf
=============================================================================*/

half4 ComputePlanarReflections(float3 TranslatedWorldPosition, half3 WorldNormal, half Roughness)
{
	half4 OutPlanarReflection = 0;

	float PlaneDistance = dot(PlanarReflectionStruct.ReflectionPlane, float4(TranslatedWorldPosition, -1));
	half DistanceFade = 1 - saturate(abs(PlaneDistance) * PlanarReflectionStruct.PlanarReflectionParameters.x + PlanarReflectionStruct.PlanarReflectionParameters.y);

	float3 PlaneOriginToWorldPosition = TranslatedWorldPosition - PlanarReflectionStruct.PlanarReflectionOrigin.xyz;
	float XAxisDistance = dot(PlaneOriginToWorldPosition, PlanarReflectionStruct.PlanarReflectionXAxis.xyz);
	half XAxisFade = saturate((PlanarReflectionStruct.PlanarReflectionXAxis.w - abs(XAxisDistance)) * PlanarReflectionStruct.PlanarReflectionParameters.x);
	float YAxisDistance = dot(PlaneOriginToWorldPosition, PlanarReflectionStruct.PlanarReflectionYAxis.xyz);
	half YAxisFade = saturate((PlanarReflectionStruct.PlanarReflectionYAxis.w - abs(YAxisDistance)) * PlanarReflectionStruct.PlanarReflectionParameters.x);
	DistanceFade *= XAxisFade * YAxisFade;

	half AngleFade = saturate(dot(PlanarReflectionStruct.ReflectionPlane.xyz, WorldNormal) * PlanarReflectionStruct.PlanarReflectionParameters2.x + PlanarReflectionStruct.PlanarReflectionParameters2.y);
	half RoughnessFade = 1 - saturate((Roughness - .2f) * 10.0f);
	half FinalFade = DistanceFade * AngleFade * RoughnessFade;

	BRANCH
	if (FinalFade > 0)
	{
		// CameraToPixel in the main view is what we used as ReflectionVector when rendering the reflection pass to PlanarReflectionTexture
		float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(ResolvedView, TranslatedWorldPosition);
		// Reflect the effective ReflectionVector in mirrored space to get the original camera vector
		float3 MirroredCameraVector = reflect(CameraToPixel, -PlanarReflectionStruct.ReflectionPlane.xyz);
		// Transform the GBuffer normal into mirrored space
		half3 MirroredNormal = mul(WorldNormal, PlanarReflectionStruct.InverseTransposeMirrorMatrix).xyz;
		// Reflect the original camera vector across the GBuffer normal in mirrored space
		half3 MirroredReflectionVectorOffNormal = reflect(MirroredCameraVector, MirroredNormal);
		// At this point we have a new reflection vector off of the GBuffer normal, and we need to approximate its intersection with the scene
		// An accurate intersection would ray trace the planar reflection depth buffer
		// As an approximation we are just intersecting with a user defined sphere
		float3 VirtualReflectionSpherePosition = TranslatedWorldPosition + MirroredReflectionVectorOffNormal * PlanarReflectionStruct.PlanarReflectionParameters.z;
		// Transform the intersection position into view space
		float3 ViewVirtualReflectionSpherePosition = mul(float4(VirtualReflectionSpherePosition, 1), ResolvedView.TranslatedWorldToView).xyz;
		// Transform the intersection position into clip space using the same projection matrix used to render PlanarReflectionTexture
		float4 ClipVirtualReflectionSpherePosition = mul(float4(ViewVirtualReflectionSpherePosition, 1), PlanarReflectionStruct.ProjectionWithExtraFOV[ResolvedView.StereoPassIndex]);

		uint EyeIndex = 0;
		if (PlanarReflectionStruct.bIsStereo)
		{
			EyeIndex = ResolvedView.StereoPassIndex;
		}

		half2 NDC = clamp(ClipVirtualReflectionSpherePosition.xy / ClipVirtualReflectionSpherePosition.w, -PlanarReflectionStruct.PlanarReflectionScreenBound, PlanarReflectionStruct.PlanarReflectionScreenBound);
		half2 ViewportUV = NDC * PlanarReflectionStruct.PlanarReflectionScreenScaleBias[EyeIndex].xy + PlanarReflectionStruct.PlanarReflectionScreenScaleBias[EyeIndex].zw;

		half4 PlanarReflectionTextureValue = Texture2DSampleLevel(
			PlanarReflectionStruct.PlanarReflectionTexture,
#if SUPPORTS_INDEPENDENT_SAMPLERS
			View.SharedTrilinearClampedSampler, 
#else
			PlanarReflectionStruct.PlanarReflectionSampler,
#endif
			ViewportUV, 
			0);

		// Fade out in regions of the planar reflection that weren't written to, so we can composite with other reflection methods
		FinalFade *= PlanarReflectionTextureValue.a;
		OutPlanarReflection.rgb = PlanarReflectionTextureValue.rgb * RoughnessFade; // Add roughness fade to color to provide smooth color transition.
		OutPlanarReflection.a = FinalFade;
	}

	return OutPlanarReflection;
}

#if (FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1)
half4 GetPlanarReflection(float3 TranslatedWorldPosition, half3 WorldNormal, half Roughness)
{
	half4 PlanarReflection = ComputePlanarReflections(TranslatedWorldPosition, WorldNormal, Roughness);
#if OUTPUT_GAMMA_SPACE
	// the capture will also be in gamma space, convert to linear:
	PlanarReflection.rgb *= PlanarReflection.rgb;
#endif

	return PlanarReflection;
}
#endif