// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PositionOnlyDepthOnlyVertexShader.hlsl: Depth-only vertex shader.
=============================================================================*/

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

void Main(
	FPositionOnlyVertexFactoryInput Input,
	out INVARIANT_OUTPUT float4 OutPosition : SV_POSITION,
	out FStereoVSOutput StereoOutput
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, StereoOutput);

	float4 WorldPos = VertexFactoryGetWorldPosition(Input);

	{
		OutPosition = INVARIANT(mul(WorldPos, ResolvedView.TranslatedWorldToClip));
	}

#if USE_GLOBAL_CLIP_PLANE
	OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(WorldPos.xyz, 1));
#endif
}
