// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PositionReconstructionCommon.ush: utils functions used in multiple ray generation shaders
=============================================================================*/

#pragma once

float3 ReconstructTranslatedWorldPositionFromDeviceZ(uint2 PixelCoord, float DeviceZ)
{
	float4 TranslatedWorldPosition = mul(float4(PixelCoord + 0.5, DeviceZ, 1), View.SVPositionToTranslatedWorld);
	TranslatedWorldPosition.xyz /= TranslatedWorldPosition.w;
	return TranslatedWorldPosition.xyz;
}

void ReconstructTranslatedWorldPositionAndCameraDirectionFromDeviceZ(uint2 PixelCoord, float DeviceZ, out float3 OutTranslatedWorldPosition, out float3 OutCameraDirection)
{
	float4 TranslatedWorldPosition = mul(float4(PixelCoord + 0.5, DeviceZ, 1), View.SVPositionToTranslatedWorld);
	TranslatedWorldPosition.xyz /= TranslatedWorldPosition.w;
	OutTranslatedWorldPosition = TranslatedWorldPosition.xyz;
	OutCameraDirection = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition.xyz);
}

float3 ReconstructTranslatedWorldPositionFromDepth(float2 UV, float SceneDepth)
{
	float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float4 HomogeneousWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld);
	float3 TranslatedWorldPosition = HomogeneousWorldPosition.xyz / HomogeneousWorldPosition.w;
	return TranslatedWorldPosition;
}

float3 ReconstructWorldPositionFromDepth(float2 UV, float Depth)
{
	float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float4 HomogeneousWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, Depth), Depth, 1), DFHackToFloat(PrimaryView.ScreenToWorld));
	float3 WorldPosition = HomogeneousWorldPosition.xyz / HomogeneousWorldPosition.w;

	return WorldPosition;
}
