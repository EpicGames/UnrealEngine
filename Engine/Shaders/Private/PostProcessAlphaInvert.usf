// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessAlphaInvert.usf: Invert the alpha values.
	This can be used to convert from Unreal standard 0=opaque 1=transparent to the much more common 
	standard of 1=opaque 0=transparent.
=============================================================================*/

#include "Common.ush"

#if MOBILE_MULTI_VIEW
Texture2DArray InTexture;
#else
Texture2D InTexture;
#endif

SamplerState InTextureSampler;

void AlphaInvert_MainPS(
	FScreenVertexOutput								Input,
	out				 float4 OutColor				: SV_Target0
#if MOBILE_MULTI_VIEW && !INSTANCED_STEREO
	, in nointerpolation uint MultiViewId			: SV_ViewID
#endif
)
{
	uint ArrayIndex = 0;
#if MOBILE_MULTI_VIEW  && !INSTANCED_STEREO
	ArrayIndex = MultiViewId;
#endif
	
	float4 SceneColor;
	#if MOBILE_MULTI_VIEW
		SceneColor = Texture2DArraySample(InTexture, InTextureSampler, float3(Input.UV.xy,ArrayIndex));
	#else
		SceneColor = Texture2DSample(InTexture, InTextureSampler, Input.UV);
	#endif
	
	OutColor.rgb = SceneColor.rgb;
	OutColor.a = 1.0 - SceneColor.a;
}
