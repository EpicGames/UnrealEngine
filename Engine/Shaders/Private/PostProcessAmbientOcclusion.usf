// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostprocessAmbientOcclusion.usf: To generate ambient occlusion as a postprocess
=============================================================================*/

#include "Common.ush"	
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"				
#include "DeferredShadingCommon.ush"
#include "Substrate/Substrate.ush"

// set by C++:
//
// 0:low / 1: medium / 2:high / 4:very high
// SHADER_QUALITY
//
// 0:no / 1:yes
// USE_AO_SETUP_AS_INPUT
//
// 0:no / 1:yes
// USE_UPSAMPLE


#define GTAO_THICKNESS_HEURISTIC 1
// 0: AABB Clipping / 1: Clipping based on first order moment
#define GTAO_VARIANCE_CLIPPING 1

// 0: classic with weighted sample, 1: don't normalize and adjust the formula to be simpler and faster - can look better and is cheaper (Alchemy like?)
#define OPTIMIZATION_O1 1

// 1:lowest quality, 2:medium , 3:high, more doesn't give too much (maybe HZB mip computations should `be adjusted)
//#define SAMPLE_STEPS 3

// 0:off / 1:show samples on the right side of the screen
#define DEBUG_LOOKUPS 0

// 0:off / 1:take into account scene normals in the computations
#define USE_NORMALS 1

// useful to remove high frequency dither pattern, not that needed with more sample
// 0:off (fast but dither pattern with low sample count), 1:non normal aware (half res look), 2:normal aware (slower), 3:normal and depth aware (slowest, doesn't add much)
//#define QUAD_MESSAGE_PASSING_BLUR 2

// ambient occlusion
// AO_SAMPLE_QUALITY = 0 : no AO sampling, only upsampling
// AO_SAMPLE_QUALITY = 1 : no dither/per pixel randomization
// AO_SAMPLE_QUALITY = 2 : efficient high frequency 4x4 pattern without jitter for TemporalAA
// AO_SAMPLE_QUALITY = 3 : efficient high frequency 4x4 pattern with jitter for TemporalAA

// SHADER_QUALITY 0-4
#if SHADER_QUALITY == 0
	// very low
	#define USE_SAMPLESET 1
	#define SAMPLE_STEPS 1
	#define QUAD_MESSAGE_PASSING_BLUR 0
#elif SHADER_QUALITY == 1
	// low
	#define USE_SAMPLESET 1
	#define SAMPLE_STEPS 1
	#define QUAD_MESSAGE_PASSING_BLUR 2
#elif SHADER_QUALITY == 2
	// medium
	#define USE_SAMPLESET 1
	#define SAMPLE_STEPS 2
	#define QUAD_MESSAGE_PASSING_BLUR 2
#elif SHADER_QUALITY == 3
	// high
	#define USE_SAMPLESET 1
	#define SAMPLE_STEPS 3
	#define QUAD_MESSAGE_PASSING_BLUR 0
#else // SHADER_QUALITY == 4
	// very high
	#define USE_SAMPLESET 3
	#define SAMPLE_STEPS 3
	#define QUAD_MESSAGE_PASSING_BLUR 0
#endif

#if QUAD_MESSAGE_PASSING_BLUR == 0
	#define QUAD_MESSAGE_PASSING_NORMAL 0
	#define QUAD_MESSAGE_PASSING_DEPTH 0
#elif QUAD_MESSAGE_PASSING_BLUR == 1
	#define QUAD_MESSAGE_PASSING_NORMAL 0
	#define QUAD_MESSAGE_PASSING_DEPTH 0
#elif QUAD_MESSAGE_PASSING_BLUR == 2
	#define QUAD_MESSAGE_PASSING_NORMAL 1
	#define QUAD_MESSAGE_PASSING_DEPTH 0
#elif QUAD_MESSAGE_PASSING_BLUR == 3
	#define QUAD_MESSAGE_PASSING_NORMAL 1
	#define QUAD_MESSAGE_PASSING_DEPTH 1
#endif

// 0:4 samples, 1:9 samples (only really noticable with dither usage ??)
//#define AO_UPSAMPLE_QUALITY 

#if USE_AO_SETUP_AS_INPUT == 1
	// lower resolution
	#define AO_SAMPLE_QUALITY 3
	#undef USE_SAMPLESET
	#define USE_SAMPLESET 3
	#define AO_UPSAMPLE_QUALITY 1
#else
	// full resolution is expensive, do lower quality
	#define AO_SAMPLE_QUALITY 3
	#define AO_UPSAMPLE_QUALITY 0
#endif

// 0: 1 point (for testing)
// 1: 3 points
// 2: more evenly spread (5 points - slightly faster, stronger effect, better with multiple levels?)
// 3: near the surface very large, softly fading out (6 points)
#if USE_SAMPLESET == 0
	#define SAMPLESET_ARRAY_SIZE 1
	static const float2 OcclusionSamplesOffsets[SAMPLESET_ARRAY_SIZE]=
	{
		// one sample, for testing
		float2(0.500, 0.500), 
	};
#elif USE_SAMPLESET == 1
	#define SAMPLESET_ARRAY_SIZE 3
	static const float2 OcclusionSamplesOffsets[SAMPLESET_ARRAY_SIZE]=
	{
		// 3 points distributed on the unit disc, spiral order and distance
		float2(0, -1.0f) * 0.43f, 
		float2(0.58f, 0.814f) * 0.7f, 
		float2(-0.58f, 0.814f) 
	};
#elif USE_SAMPLESET == 2
	#define SAMPLESET_ARRAY_SIZE 5
	static const float2 OcclusionSamplesOffsets[SAMPLESET_ARRAY_SIZE]=
	{
		// 5 points distributed on a ring
		float2(0.156434, 0.987688),
		float2(0.987688, 0.156434)*0.9,
		float2(0.453990, -0.891007)*0.8,
		float2(-0.707107, -0.707107)*0.7,
		float2(-0.891006, 0.453991)*0.65,
	};
#else // USE_SAMPLESET == 3
	#define SAMPLESET_ARRAY_SIZE 6
	static const float2 OcclusionSamplesOffsets[SAMPLESET_ARRAY_SIZE]=
	{
		// 6 points distributed on the unit disc, spiral order and distance
		float2(0.000, 0.200), 
		float2(0.325, 0.101), 
		float2(0.272, -0.396), 
		float2(-0.385, -0.488), 
		float2(-0.711, 0.274), 
		float2(0.060, 0.900) 
	};
#endif // USE_SAMPLESET

float3 PackSceneDepth(float InSceneDepth)
{
	return UnpackRGBA8(PackR24F(InSceneDepth)).rgb;
}

float UnpackSceneDepth(float3 InPackedSceneDepth)
{
	return UnpackR24F(PackRGBA8(float4(InPackedSceneDepth, 0)));
}

// -----------------------------------------------------------------------------------------------------------------------------
// To be included after defines
#include "PostProcessAmbientOcclusionCommon.ush"

#if UPSAMPLE_FILTER_SHADER

/*
 *    UPSAMPLE FILTER
 *
 */
Texture2D GTAOUpsampleTexture;
SamplerState GTAOUpsampleSampler;
float2 GTAOUpsamplePixelSize;


float GTAOUpsamplePSAndCS(float2 UV)
{
	float2 Offset = GTAOUpsamplePixelSize * 0.25;

	float AOC = saturate(Texture2DSample(GTAOUpsampleTexture, GTAOUpsampleSampler, UV).r);
	float AO0 = saturate(Texture2DSample(GTAOUpsampleTexture, GTAOUpsampleSampler, UV + float2(-Offset.x, -Offset.y)).r);
	float AO1 = saturate(Texture2DSample(GTAOUpsampleTexture, GTAOUpsampleSampler, UV + float2(Offset.x, -Offset.y)).r);
	float AO2 = saturate(Texture2DSample(GTAOUpsampleTexture, GTAOUpsampleSampler, UV + float2(-Offset.x, Offset.y)).r);
	float AO3 = saturate(Texture2DSample(GTAOUpsampleTexture, GTAOUpsampleSampler, UV + float2(Offset.x, Offset.y)).r);

	float AO = min(min(AO0, AO1), min(AO2, AO3));
	return AO;
}

void GTAOUpsamplePS(in noperspective float4 UVAndScreenPos : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = GTAOUpsamplePSAndCS(UVAndScreenPos.xy);
}

#else // UPSAMPLE_FILTER_SHADER

// downsample the input of the ambient occlusion pass for better performance, can take input from setup or another downsample pass
void MainSetupPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor0 : SV_Target0)
{
	StereoSetupPS(StereoInput);

	float2 ViewPortSize = AOViewport_ViewportSize;
	float2 InUV = UVAndScreenPos.xy;

	// can be optimized
	float2 UV[4];
	UV[0] = InUV + float2(-0.5f, -0.5f) * InputExtentInverse;
	UV[1] = min(InUV + float2( 0.5f, -0.5f) * InputExtentInverse, View.BufferBilinearUVMinMax.zw);
	UV[2] = min(InUV + float2(-0.5f,  0.5f) * InputExtentInverse, View.BufferBilinearUVMinMax.zw);
	UV[3] = min(InUV + float2( 0.5f,  0.5f) * InputExtentInverse, View.BufferBilinearUVMinMax.zw);

	float4 Samples[4];
	
	UNROLL for(uint i = 0; i < 4; ++i)
	{
#if COMPUTE_SHADER || FORWARD_SHADING
		// Async compute and forward shading don't have access to the gbuffer.
		Samples[i].rgb = normalize(ReconstructNormalFromDepthBuffer(float4(UV[i] * ViewPortSize, SvPosition.zw))) * 0.5f + 0.5f;
#else
#if SUBSTRATE_GBUFFER_FORMAT==1
		const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(clamp(UV[i] * View.BufferSizeAndInvSize.xy, 0.0f, View.BufferSizeAndInvSize.xy - 1.0f), 0)));
		Samples[i].rgb = TopLayerData.WorldNormal * 0.5f + 0.5f;
#else
		Samples[i].rgb = GetGBufferData(UV[i], true).WorldNormal * 0.5f + 0.5f;
#endif
#endif
		Samples[i].a = CalcSceneDepth(UV[i]);
	}
	
	float MaxZ = max( max(Samples[0].a, Samples[1].a), max(Samples[2].a, Samples[3].a));

	float4 AvgColor = 0.0f;
	if (USE_NORMALS)
	{
		AvgColor = 0.0001f;

		{
			UNROLL for(uint i = 0; i < 4; ++i)
			{
				AvgColor += float4(Samples[i].rgb, 1) * ComputeDepthSimilarity(Samples[i].a, MaxZ, ThresholdInverse);
			}
			AvgColor.rgb /= AvgColor.w;
		}
	}

	OutColor0 = float4(AvgColor.rgb, MaxZ / Constant_Float16F_Scale);
}

// the main pixel shader that computes ambient occlusion
void MainPSandCS(in float4 UVAndScreenPos, float4 SvPosition, out float4 OutColor)
{
	OutColor = 0;

	// the following constants as set up on C++ side
	float AmbientOcclusionPower = ScreenSpaceAOParams[0].x;
	float Ratio = ScreenSpaceAOParams[1].w;
	float AORadiusInShader = ScreenSpaceAOParams[1].z;
	float InvAmbientOcclusionDistance = ScreenSpaceAOParams[0].z;
	float AmbientOcclusionIntensity = ScreenSpaceAOParams[0].w;
	float2 ViewportUVToRandomUV = ScreenSpaceAOParams[1].xy;
	float AmbientOcclusionBias = ScreenSpaceAOParams[0].y;
	float ScaleFactor = ScreenSpaceAOParams[2].x;
	float ScaleRadiusInWorldSpace = ScreenSpaceAOParams[2].z;

	float2 UV = UVAndScreenPos.xy;
	float2 ScreenPos = UVAndScreenPos.zw;

	float InvTanHalfFov = ScreenSpaceAOParams[3].w;
	float3 FovFix = float3(InvTanHalfFov, Ratio * InvTanHalfFov, 1);
	float3 InvFovFix = 1.0f / FovFix;
	float4 ModifiedSvPosition = float4(SvPosition.xy * SSAO_SvPositionScaleBias.xx + SSAO_SvPositionScaleBias.yy, SvPosition.zw);
		
	float SceneDepth = GetDepthFromAOInput(UV);
	float3 WorldNormal = GetWorldSpaceNormalFromAOInput(UV, ModifiedSvPosition);

	// can be NaN if WorldNormal=0,0,0 which happens when !USE_NORMALS
	float3 ViewSpaceNormal = normalize(mul(WorldNormal, (float3x3)View.TranslatedWorldToView));

	float3 ViewSpacePosition = ReconstructCSPos(SceneDepth, ScreenPos);

	float ActualAORadius = AORadiusInShader * lerp(SceneDepth, 1, ScaleRadiusInWorldSpace);

	// Add bias after fixup (causes minor banding - not needed with larger radius)
	if (USE_NORMALS)
	{
		ViewSpacePosition += AmbientOcclusionBias * SceneDepth * ScaleFactor * (ViewSpaceNormal * FovFix);
	}

	float2 WeightAccumulator = 0.0001f;
	
	// if the feature is enabled and right side of screen
	const bool bDebugLookups = DEBUG_LOOKUPS && ViewSpacePosition.x > 0;

#if AO_SAMPLE_QUALITY != 0
	// no SSAO in this pass, only upsampling

#if AO_SAMPLE_QUALITY == 1
	// no 4x4 randomization
	float2 RandomVec = float2(0, 1) * ActualAORadius;
	{
#elif AO_SAMPLE_QUALITY == 2
	// extract one of 16 base vectors (rotation and scale) from a texture that repeats 4x4
	float2 RandomVec = (Texture2DSample(RandomNormalTexture, RandomNormalTextureSampler, UV * ViewportUVToRandomUV).rg * 2 - 1) * ActualAORadius;
	{
#else // AO_SAMPLE_QUALITY == 3
	// extract one of 16 base vectors (rotation and scale) from a texture that repeats 4x4, changing over time if TemporalAA is enabled

	// jitter each frame a bit to get higher quality over multiple frames (only if TemporalAA is enabled), can cause ghosting effects
	const float2 TemporalOffset = ScreenSpaceAOParams[3].xy;

	float2 RandomVec = (Texture2DSample(RandomNormalTexture, RandomNormalTextureSampler, TemporalOffset + UV * ViewportUVToRandomUV).rg * 2 - 1) * ActualAORadius;
	{
#endif // AO_SAMPLE_QUALITY == 

		if(bDebugLookups && ViewSpacePosition.y > 0)
		{
			// top sample are not per pixel rotated
			RandomVec = float2(0, 1) * ActualAORadius;
		}

		float2 FovFixXY = FovFix.xy * (1.0f / ViewSpacePosition.z);
		float4 RandomBase = float4(RandomVec, -RandomVec.y, RandomVec.x) * float4(FovFixXY, FovFixXY);

		float2 ScreenSpacePos = ViewSpacePosition.xy / ViewSpacePosition.z;

		// to debug the input depth
//		OutColor = GetDepthForSSAO(ScreenSpacePos, 0); return;
		// to debug the reconstructed normal
//		OutColor = ReconstructedViewSpaceNormal.z; return;

		// .x means for very anisotropic viewports we scale by x
		float InvHaloSize = 1.0f / (ActualAORadius * FovFixXY.x * 2);

		float3 ScaledViewSpaceNormal = ViewSpaceNormal;

#if OPTIMIZATION_O1
		ScaledViewSpaceNormal *= 0.08f * lerp(SceneDepth, 1000, ScaleRadiusInWorldSpace);
#endif

		UNROLL for(int i = 0; i < SAMPLESET_ARRAY_SIZE; ++i)
		{
			// -1..1
			float2 UnrotatedRandom = OcclusionSamplesOffsets[i].xy;

			float2 LocalRandom = (UnrotatedRandom.x * RandomBase.xy + UnrotatedRandom.y * RandomBase.zw);

			if (bDebugLookups)
			{
				UNROLL for(uint step = 0; step < SAMPLE_STEPS; ++step)
				{
					float Scale = (step + 1) / (float)SAMPLE_STEPS;
					float MipLevel = ComputeMipLevel(i, step);
					float2 ScaledLocalRandom = Scale * LocalRandom;
					
					WeightAccumulator += float2(ComputeSampleDebugMask(ScreenSpacePos + ScaledLocalRandom, MipLevel), 1.0f);
					WeightAccumulator += float2(ComputeSampleDebugMask(ScreenSpacePos - ScaledLocalRandom, MipLevel), 1.0f);
				}
			}
			else if (USE_NORMALS)
			{
				float3 LocalAccumulator = 0;

				UNROLL for(uint step = 0; step < SAMPLE_STEPS; ++step)
				{
					// constant at run time
					float Scale = (step + 1) / (float)SAMPLE_STEPS;
					// constant at run time (higher is better for texture cache / performance, lower is better quality
					float MipLevel = ComputeMipLevel(i, step);

					float3 StepSample = WedgeWithNormal(ScreenSpacePos, Scale * LocalRandom, InvFovFix, ViewSpacePosition, ScaledViewSpaceNormal, InvHaloSize, MipLevel);

					// combine horizon samples
					LocalAccumulator = lerp(LocalAccumulator, float3(max(LocalAccumulator.xy, StepSample.xy), 1), StepSample.z);
				}

				// Square(): the area scales quadratic with the angle - it gets a bit darker
				WeightAccumulator += float2(Square(1 - LocalAccumulator.x) * LocalAccumulator.z, LocalAccumulator.z);
				WeightAccumulator += float2(Square(1 - LocalAccumulator.y) * LocalAccumulator.z, LocalAccumulator.z);
				// cheaper? Could move 1 - out
				// WeightAccumulator += float2(1 - LocalAccumulator.x, LocalAccumulator.y);
			}
			else // Case with no normals
			{
				float2 LocalAccumulator = 0;

				UNROLL for(uint step = 0; step < SAMPLE_STEPS; ++step)
				{
					// constant at run time
					float Scale = (step + 1) / (float)SAMPLE_STEPS;
					// constant at run time (higher is better for texture cache / performance, lower is better quality
					float MipLevel = ComputeMipLevel(i, step);

					float2 StepSample = WedgeNoNormal(ScreenSpacePos, Scale * LocalRandom, InvFovFix, ViewSpacePosition, InvHaloSize, MipLevel);

					// combine horizon samples
					LocalAccumulator = lerp(LocalAccumulator, float2(max(LocalAccumulator.x, StepSample.x), 1), StepSample.y);
				}

				// Square(): the area scales quadratic with the angle - it gets a bit darker
				WeightAccumulator += float2(Square(1 - LocalAccumulator.x) * LocalAccumulator.y, LocalAccumulator.y);

			}
		}
	}

#endif // #if AO_SAMPLE_QUALITY == 0


	OutColor.r = WeightAccumulator.x / WeightAccumulator.y;
	OutColor.gb = float2(0, 0);

	if(!bDebugLookups)
	{
#if COMPUTE_SHADER || FORWARD_SHADING
		// In compute, Input1 and Input2 are not necessarily valid.
		float4 Filtered = 1;
#else
		float4 Filtered = ComputeUpsampleContribution(SceneDepth, UV, WorldNormal);
#endif
		// recombined result from multiple resolutions
		OutColor.r = lerp(OutColor.r, Filtered.r, ComputeLerpFactor());
	}

#if !USE_AO_SETUP_AS_INPUT
	if(!bDebugLookups)
	{
		// full res

		// soft fade out AO in the distance
		{
			float Mul = ScreenSpaceAOParams[4].x;
			float Add = ScreenSpaceAOParams[4].y;
			OutColor.r = lerp(OutColor.r, 1, saturate(SceneDepth * Mul + Add));
		}

		// user adjust AO
		// abs() to prevent shader warning
		OutColor.r = 1 - (1 - pow(abs(OutColor.r), AmbientOcclusionPower)) * AmbientOcclusionIntensity;

		// we output in a single alpha channel
		OutColor = OutColor.r;
	}
	else
	{
		OutColor.r = pow(1 - OutColor.r, 16);	// constnt is tweaked with radius and sample count
	}
#endif

	// we don't support ddx_fine() for SM4
#if !COMPUTE_SHADER && QUAD_MESSAGE_PASSING_BLUR > 0
	{
		// .x: AO output, .y:SceneDepth .zw:view space normal
		float4 CenterPixel = float4(OutColor.r, SceneDepth, normalize(ViewSpaceNormal).xy); 

		float4 dX = ddx_fine(CenterPixel);
		float4 dY = ddy_fine(CenterPixel);

		int2 Mod = (uint2)(SvPosition.xy) % 2;

		float4 PixA = CenterPixel;
		float4 PixB = CenterPixel - dX * (Mod.x * 2 - 1);
		float4 PixC = CenterPixel - dY * (Mod.y * 2 - 1);

		float WeightA = 1.0f;
		float WeightB = 1.0f;
		float WeightC = 1.0f;

#if QUAD_MESSAGE_PASSING_NORMAL
		const float NormalTweak = 4.0f;
		float3 NormalA = ReconstructNormal(PixA.zw);
		float3 NormalB = ReconstructNormal(PixB.zw);
		float3 NormalC = ReconstructNormal(PixC.zw);
		WeightB *= saturate(pow(saturate(dot(NormalA, NormalB)), NormalTweak));
		WeightC *= saturate(pow(saturate(dot(NormalA, NormalC)), NormalTweak));
#endif

#if QUAD_MESSAGE_PASSING_DEPTH
		const float DepthTweak = 1;
		float InvDepth = 1.0f / PixA.y;
		WeightB *= 1 - saturate(abs(1 - PixB.y * InvDepth) * DepthTweak);
		WeightC *= 1 - saturate(abs(1 - PixC.y * InvDepth) * DepthTweak);
#endif

		// + 1.0f to avoid div by 0
		float InvWeightABC = 1.0f / (WeightA + WeightB + WeightC);

		WeightA *= InvWeightABC;
		WeightB *= InvWeightABC;
		WeightC *= InvWeightABC;

		OutColor = WeightA * PixA.x + WeightB * PixB.x + WeightC * PixC.x;
		// visualize where we don't want to fade
//		OutColor = (WeightA - 0.333f) / 0.666f;
	}
#endif

#if OUTPUT_DEPTH
	OutColor.gba = PackSceneDepth(SceneDepth);
#endif
}

void MainPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	MainPSandCS(UVAndScreenPos, SvPosition, OutColor);	
}

#if COMPUTE_SHADER
/** Output target. In compute, this is a single value buffer. */
RWTexture2D<float> OutTexture;
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
	float ScaleFactor = ScreenSpaceAOParams[2].x;
	
	int2 PixelPos = DispatchThreadId + AOViewport_ViewportMin; 
	float2 PixelCenter = (float2)PixelPos + float2(0.5, 0.5);
	
	// todo: move to a function
	float4 SvPosition = float4(PixelCenter, 0, 0) * ScaleFactor;	
	float2 BufferUV = PixelCenter * AOViewport_ExtentInverse;
	float2 ScreenPos = UVToScreenPos(BufferUV, AOViewport_Extent);

	SvPosition.z = LookupDeviceZ(BufferUV);
	// todo: investigate
//  SvPosition.w = ConvertFromDeviceZ(SvPosition.z);
	SvPosition.w = 1;

	float4 OutColor = 1;

	// Test for early exit with out of depth bound.
	float SceneDepth = ConvertFromDeviceZ(SvPosition.z);
	float FadeMul = ScreenSpaceAOParams[4].x;
	float FadeAdd = ScreenSpaceAOParams[4].y;
	BRANCH
	if (SceneDepth * FadeMul + FadeAdd < 1)
	{
		MainPSandCS(float4(BufferUV, ScreenPos), SvPosition, OutColor);
	}

	// Here we could optimized for coalessing writes but that might not be the performance bottleneck.
	// We should rather optimized for best texture cache performance.
	// http://on-demand.gputechconf.com/gtc/2010/presentations/S12312-DirectCompute-Pre-Conference-Tutorial.pdf
	OutTexture[PixelPos] = OutColor.r;
}
#endif

SCREEN_PASS_TEXTURE_VIEWPORT(SSAOSmoothOutputViewport)
FScreenTransform SSAOSmoothOutputToInput;

Texture2D SSAOSmoothInputTexture;
SamplerState SSAOSmoothInputSampler;

float SSAOSmoothPSandCS(in float2 UV)
{
	float2 SampleUV = ApplyScreenTransform(UV, SSAOSmoothOutputToInput);

	// Use a 4x4 box filter because the random texture is tiled 4x4
	float Result;
	Result = SSAOSmoothInputTexture.SampleLevel(SSAOSmoothInputSampler, SampleUV, 0).r;
	Result += SSAOSmoothInputTexture.SampleLevel(SSAOSmoothInputSampler, SampleUV, 0, int2(2, 0)).r;
	Result += SSAOSmoothInputTexture.SampleLevel(SSAOSmoothInputSampler, SampleUV, 0, int2(0, 2)).r;
	Result += SSAOSmoothInputTexture.SampleLevel(SSAOSmoothInputSampler, SampleUV, 0, int2(2, 2)).r;

	return Result * 0.25;
}

void MainSSAOSmoothPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float SmoothColor = SSAOSmoothPSandCS(UVAndScreenPos.xy);
	OutColor = SmoothColor;
}

#if COMPUTE_SHADER
RWTexture2D<float> SSAOSmoothOutputTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainSSAOSmoothCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID)
{
	BRANCH
	if (any(DispatchThreadId >= (uint2)SSAOSmoothOutputViewport_ViewportSize))
	{
		return;
	}

	uint2 DestPixelPos = SSAOSmoothOutputViewport_ViewportMin + DispatchThreadId;
	float2 DestUV = DestPixelPos * SSAOSmoothOutputViewport_ExtentInverse;

	SSAOSmoothOutputTexture[DestPixelPos] = SSAOSmoothPSandCS(DestUV);
}
#endif


#ifdef UPSAMPLE_PASS

Texture2D AOTexture;
SamplerState AOSampler;

float UpsampleSSAO(float2 BufferUV)
{
#if UPSAMPLE_QUALITY == 0
	// Bilinear sample.
	return Texture2DSampleLevel(AOTexture, AOSampler, BufferUV, 0).x;

#elif UPSAMPLE_QUALITY == 1
	// 4 sample bilinear with depth weighting.
	float2 LowResBufferSize = floor(View.BufferSizeAndInvSize.xy / 2);
	float2 LowResTexelSize = 1.0f / LowResBufferSize;

	float2 Corner00UV = floor(BufferUV * LowResBufferSize - 0.5f) / LowResBufferSize + 0.5f * LowResTexelSize;
	float2 BilinearWeights = (BufferUV - Corner00UV) * LowResBufferSize;

	float4 AOValues00 = Texture2DSampleLevel(AOTexture, AOSampler, Corner00UV, 0);
	float4 AOValues10 = Texture2DSampleLevel(AOTexture, AOSampler, Corner00UV + float2(LowResTexelSize.x, 0), 0);
	float4 AOValues01 = Texture2DSampleLevel(AOTexture, AOSampler, Corner00UV + float2(0, LowResTexelSize.y), 0);
	float4 AOValues11 = Texture2DSampleLevel(AOTexture, AOSampler, Corner00UV + LowResTexelSize, 0);

	float4 CornerWeights = float4(
		(1 - BilinearWeights.y) * (1 - BilinearWeights.x),
		(1 - BilinearWeights.y) * BilinearWeights.x,
		BilinearWeights.y * (1 - BilinearWeights.x),
		BilinearWeights.y * BilinearWeights.x);

	float SceneDepth00 = UnpackSceneDepth(AOValues00.gba);
	float SceneDepth10 = UnpackSceneDepth(AOValues10.gba);
	float SceneDepth01 = UnpackSceneDepth(AOValues01.gba);
	float SceneDepth11 = UnpackSceneDepth(AOValues11.gba);

	float MaxDepth = max(max(max(SceneDepth00, SceneDepth10), SceneDepth01), SceneDepth11);
	float MinDepth = min(min(min(SceneDepth00, SceneDepth10), SceneDepth01), SceneDepth11);

	if (MaxDepth / MinDepth > 1.02f)
	{
		float Epsilon = 0.0001f;
		float SceneDepth = GetDepthFromAOInput(BufferUV);
		float4 CornerDepths = abs(float4(SceneDepth00, SceneDepth10, SceneDepth01, SceneDepth11));
		float4 DepthWeights = 1.0f / (abs(CornerDepths - SceneDepth.xxxx) + Epsilon);
		CornerWeights *= DepthWeights;
	}

	float InterpolatedResult =
		(CornerWeights.x * AOValues00.r
			+ CornerWeights.y * AOValues10.r
			+ CornerWeights.z * AOValues01.r
			+ CornerWeights.w * AOValues11.r)
		/ dot(CornerWeights, 1);

	return InterpolatedResult;

#elif UPSAMPLE_QUALITY == 2
	// 9 sample with depth weighting.
	float2 LowResBufferSize = floor(View.BufferSizeAndInvSize.xy * 0.5);
	float2 LowResTexelSize = 1.0f / LowResBufferSize;

	float4 AOValues[9];
	AOValues[0] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(-1, -1) * LowResTexelSize, 0);
	AOValues[1] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(0, -1) * LowResTexelSize, 0);
	AOValues[2] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(1, -1) * LowResTexelSize, 0);
	AOValues[3] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(-1, 0) * LowResTexelSize, 0);
	AOValues[4] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(0, 0) * LowResTexelSize, 0);
	AOValues[5] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(1, 0) * LowResTexelSize, 0);
	AOValues[6] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(-1, 1) * LowResTexelSize, 0);
	AOValues[7] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(0, 1) * LowResTexelSize, 0);
	AOValues[8] = Texture2DSampleLevel(AOTexture, AOSampler, BufferUV + float2(1, 1) * LowResTexelSize, 0);

	float SceneDepth = GetDepthFromAOInput(BufferUV);
	float Epsilon = 0.0001f;

	float TotalSum = 0;
	float TotalWeight = 0;
	UNROLL for (int i = 0; i < 9; ++i)
	{
		float SampleValue = AOValues[i].r;
		float SampleDepth = UnpackSceneDepth(AOValues[i].gba);

		float Weight = 1.0f / (abs(SceneDepth - SampleDepth) + Epsilon);
		TotalWeight += Weight;
		TotalSum += SampleValue * Weight;
	}

	return TotalSum / TotalWeight;
#endif
}

void AmbientOcclusionUpsamplePS(
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	OutColor = UpsampleSSAO(BufferUV).xxxx;
}

#endif // UPSAMPLE_PASS

#if SHADER_QUALITY == 0
	// very low
	#define GTAO_NUMTAPS 4
	#define GTAO_BIASMIPLEVEL 2
	#define GTAO_MAX_PIXEL_SCREEN_RADIUS 256.0f
#elif SHADER_QUALITY == 1
	// low
	#define GTAO_NUMTAPS 6
	#define GTAO_BIASMIPLEVEL 1
	#define GTAO_MAX_PIXEL_SCREEN_RADIUS 256.0f
#elif SHADER_QUALITY == 2
	// medium
	#define GTAO_NUMTAPS 8
	#define GTAO_BIASMIPLEVEL 0
	#define GTAO_MAX_PIXEL_SCREEN_RADIUS 256.0f
#elif SHADER_QUALITY == 3
	// high
	#define GTAO_NUMTAPS 12
	#define GTAO_BIASMIPLEVEL 0
	#define GTAO_MAX_PIXEL_SCREEN_RADIUS 256.0f
#else // SHADER_QUALITY == 4
	// very high
	#define GTAO_NUMTAPS 20
	#define GTAO_BIASMIPLEVEL 0
	#define GTAO_MAX_PIXEL_SCREEN_RADIUS 256.0f
#endif


float4 GTAOParams[5];
// [0] - { cos(TemporalAngle), sin(TemporalAngle), TemporalOffset, FrameTemporalOffset}
// [1] - { FrameNumber, Thicknessblend, unused, unused}
// [2] - { TargetSizeX, TargetSizeY, 1.0/TargetSizeX, 1.0f/TargetSizeY}
// [3] - { FallOffStart, FallOffEnd, FalloffScale, FalloffBias}
// [4] - { Temporal Blend Weight, Angles, SinDeltaAngle, CosDeltaAngle}

#define PI_HALF (PI*0.5)

#if COMPUTE_SHADER
RWTexture2D<float4> HorizonOutTexture;
RWTexture2D<float>	DepthOutTexture;
RWTexture2D<float2> VelocityOutTexture;
RWTexture2D<float> DepthsTexture;
#endif

Texture2D		HistoryTexture;
SamplerState	HistoryTextureSampler;
float2			HistoryTextureSize;
float2			HistoryTexturePixelSize;

Texture2D		ZCurrTexture;
SamplerState	ZCurrTextureSampler;

float4 PrevScreenPositionScaleBias;


float ClampScale(float Scale)
{ 
	return clamp(Scale, 2.0, 8.0);
}




float3 GetNormal(float2 UV, float3 ViewSpacePosMid)
{
	float3 ViewSpaceNormal;

#if USE_NORMALBUFFER

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(clamp(UV * View.BufferSizeAndInvSize.xy, 0.0f, View.BufferSizeAndInvSize.xy-1.0f), 0)));
	float3 WorldNormal = TopLayerData.WorldNormal;
#else
	// Get the normal from the normal buffer
	float3 WorldNormal		= GetGBufferData(UV, false).WorldNormal;
#endif
	ViewSpaceNormal	= normalize(mul(WorldNormal, (float3x3)View.TranslatedWorldToView));

#else
	// Get the normal derived from the depth buffer
	float2 DeltaUV = View.BufferSizeAndInvSize.zw;

#if SHADING_PATH_MOBILE
	Texture2D DepthTexture = MobileSceneTextures.SceneDepthTexture;
	SamplerState DepthTextureSampler = MobileSceneTextures.SceneDepthTextureSampler;
#else
	Texture2D DepthTexture = SceneTexturesStruct.SceneDepthTexture;
	SamplerState DepthTextureSampler = SceneTexturesStruct_SceneDepthTextureSampler;
#endif
	
	float DeviceZ		= Texture2DSampleLevel(DepthTexture, DepthTextureSampler, UV,0).r;
	float DeviceZLeft	= Texture2DSampleLevel(DepthTexture, DepthTextureSampler, UV + float2(-DeltaUV.x,  0.0f),0).r;
	float DeviceZTop	= Texture2DSampleLevel(DepthTexture, DepthTextureSampler, UV + float2( 0.0f     , -DeltaUV.y),0).r;
	float DeviceZRight	= Texture2DSampleLevel(DepthTexture, DepthTextureSampler, UV + float2( DeltaUV.x,  0.0f),0).r;
	float DeviceZBottom = Texture2DSampleLevel(DepthTexture, DepthTextureSampler, UV + float2( 0.0f     ,  DeltaUV.y),0).r;

	float DeviceZDdx	= TakeSmallerAbsDelta(DeviceZLeft, DeviceZ, DeviceZRight);
	float DeviceZDdy	= TakeSmallerAbsDelta(DeviceZTop, DeviceZ, DeviceZBottom);

	float ZRight		= ConvertFromDeviceZ(DeviceZ + DeviceZDdx);
	float ZDown			= ConvertFromDeviceZ(DeviceZ + DeviceZDdy);

	float3 Right		= ScreenToViewPos(UV+ float2( DeltaUV.x,      0.0f) , ZRight)-ViewSpacePosMid;
	float3 Down			= ScreenToViewPos(UV+ float2(      0.0f, DeltaUV.y) , ZDown) -ViewSpacePosMid;
		
	ViewSpaceNormal = normalize(cross(Right, Down));
#endif

	return ViewSpaceNormal;
}


float GetLinearDepthProj(float2 ScreenUV)
{
#if SHADING_PATH_MOBILE
	Texture2D DepthTexture = MobileSceneTextures.SceneDepthTexture;
	SamplerState DepthTextureSampler = MobileSceneTextures.SceneDepthTextureSampler;
#else
	Texture2D DepthTexture = SceneTexturesStruct.SceneDepthTexture;
	SamplerState DepthTextureSampler = SceneTexturesStruct_SceneDepthTextureSampler;
#endif
	
	float DeviceZ = Texture2DSampleLevel(DepthTexture, DepthTextureSampler, ScreenUV, 0).r;
	return  1.0f / (DeviceZ * View.InvDeviceZToWorldZTransform[2] - View.InvDeviceZToWorldZTransform[3]);
}

float2 SearchForLargestAngleDual(uint NumSteps, float2 BaseUV, float2 ScreenDir, float SearchRadius, float InitialOffset, float3 ViewPos, float3 ViewDir,float AttenFactor)
{
	float SceneDepth, LenSq, OOLen, Ang, FallOff;
	float3 V;
	float2 SceneDepths =0;

	float2 BestAng = float2(-1,-1);
	float Thickness = GTAOParams[1].y;

	for(uint i=0; i<NumSteps; i++)
	{
		float fi		 = (float) i;

		float2 UVOffset  = ScreenDir * max( SearchRadius * (fi + InitialOffset), (fi+1) );
		UVOffset.y		*= -1;
		float4 UV2		 = BaseUV.xyxy + float4( UVOffset.xy, -UVOffset.xy );

	// Positive Direction
		SceneDepths.x	= ConvertFromDeviceZ(LookupDeviceZ(UV2.xy));
		SceneDepths.y	= ConvertFromDeviceZ(LookupDeviceZ(UV2.zw));

		V				= ScreenToViewPos(UV2.xy, SceneDepths.x) - ViewPos;
		LenSq			= dot(V,V);
		OOLen			= rsqrt(LenSq + 0.0001);
		Ang				= dot(V,ViewDir) * OOLen;

		FallOff   	= saturate(LenSq * AttenFactor);  
		Ang			= lerp(Ang, BestAng.x, FallOff);
		BestAng.x  = ( Ang > BestAng.x ) ? Ang : lerp( Ang, BestAng.x, Thickness );  

	// Negative Direction
		V			= ScreenToViewPos(UV2.zw, SceneDepths.y) - ViewPos;
		LenSq		= dot(V,V);
		OOLen		= rsqrt(LenSq + 0.0001);
		Ang			= dot(V,ViewDir) * OOLen;

		FallOff   	= saturate(LenSq * AttenFactor);  
		Ang			= lerp(Ang, BestAng.y, FallOff);

		BestAng.y  = ( Ang > BestAng.y ) ? Ang : lerp( Ang, BestAng.y, Thickness );  
	}

	BestAng.x = acosFast(clamp(BestAng.x, -1.0,  1.0));
	BestAng.y = acosFast(clamp(BestAng.y, -1.0,  1.0));

	return BestAng;
}


float2 SearchForLargestAngleDual_HZB(uint NumSteps, float2 BaseUV, float2 ScreenDir, float SearchRadius, float InitialOffset, float3 ViewPos, float3 ViewDir, float AttenFactor)
{
	float SceneDepth, LenSq, OOLen, Ang, FallOff;
	float3 V;
	float2 SceneDepths =0;
	float MipLevel = 0 ;

	float2 BestAng = float2(-1,-1);
	float Thickness = GTAOParams[1].y;

	UNROLL 
	for(uint i=1; i<NumSteps+1; i++)
	{
		float fi		 = (float) i;

		float2 UVOffset  = ScreenDir * max( SearchRadius * (fi + InitialOffset), (fi+1) );
		UVOffset.y		*= -1;
		float4 UV2		 = BaseUV.xyxy + float4( UVOffset.xy, -UVOffset.xy );

		if( (i==0)  && (GTAO_BIASMIPLEVEL==0))
		{
			SceneDepths.x	= ConvertFromDeviceZ(LookupDeviceZ(UV2.xy));
			SceneDepths.y	= ConvertFromDeviceZ(LookupDeviceZ(UV2.zw));
		} 
		else
		{
			MipLevel = GTAO_BIASMIPLEVEL;

			if(i==2)
				MipLevel++;
	
			if(i>3)
				MipLevel+=2;

			SceneDepths.x	= GetHZBDepth(UV2.xy,MipLevel);
			SceneDepths.y	= GetHZBDepth(UV2.zw,MipLevel);
		}


	// Positive Direction
		V				= ScreenToViewPos(UV2.xy, SceneDepths.x) - ViewPos;
		LenSq			= dot(V,V);
		OOLen			= rsqrtFast(LenSq + 0.0001);
		FallOff   		= saturate(LenSq * AttenFactor);
		
		if(FallOff < 1.0)
		{
			Ang				= dot(V,ViewDir) * OOLen;
			Ang				= lerp(Ang, BestAng.x, FallOff);
			BestAng.x		= ( Ang > BestAng.x ) ? Ang : lerp( Ang, BestAng.x, Thickness );  
		}


	// Negative Direction
		V			= ScreenToViewPos(UV2.zw, SceneDepths.y) - ViewPos;
		LenSq		= dot(V,V);
		OOLen		= rsqrtFast(LenSq + 0.0001);
		FallOff   	= saturate(LenSq * AttenFactor); 

		if(FallOff < 1.0)
		{
			Ang			= dot(V,ViewDir) * OOLen;
			Ang			= lerp(Ang, BestAng.y, FallOff);
			BestAng.y  = ( Ang > BestAng.y ) ? Ang : lerp( Ang, BestAng.y, Thickness );  
		}
	}

	BestAng.x = acosFast(clamp(BestAng.x, -1.0,  1.0));
	BestAng.y = acosFast(clamp(BestAng.y, -1.0,  1.0));

	return BestAng;
}


float ComputeInnerIntegral(float2 UV, float2 Angles, float2 ScreenDir, float3 ViewDir, float3 ViewSpaceNormal, float SceneDepth)
{
	// Given the angles found in the search plane we need to project the View Space Normal onto the plane defined by the search axis and the View Direction and perform the inner integrate
	float3 PlaneNormal		= normalize(cross(float3(ScreenDir.xy,0) ,ViewDir));
	float3 Perp				= cross(ViewDir, PlaneNormal);
	float3 ProjNormal		= ViewSpaceNormal - PlaneNormal * dot(ViewSpaceNormal, PlaneNormal);

	float LenProjNormal		= length(ProjNormal) + 0.000001f;
	float RecipMag			= 1.0f / (LenProjNormal);

	float CosAng			= dot(ProjNormal, Perp) * RecipMag;	
	float Gamma				= acosFast(CosAng) - PI_HALF;				
	float CosGamma			= dot(ProjNormal, ViewDir) * RecipMag;
	float SinGamma  		= CosAng * -2.0f;					

	// clamp to normal hemisphere 
	Angles.x = Gamma + max(-Angles.x - Gamma, -(PI_HALF) );
	Angles.y = Gamma + min( Angles.y - Gamma,  (PI_HALF) );

	float AO = ( (LenProjNormal) *  0.25 * 
					    ( (Angles.x * SinGamma + CosGamma - cos((2.0 * Angles.x) - Gamma)) +
				  	      (Angles.y * SinGamma + CosGamma - cos((2.0 * Angles.y) - Gamma)) ));

	return AO;
}



float InterleavedGradientNoise( float2 iPos )
{
	return frac( 52.9829189f * frac( (iPos.x * 0.06711056) + (iPos.y*0.00583715)) );
}

float2 GetRandomAngleOffset(uint2 iPos )
{
	iPos.y = 4096-iPos.y;
	float Angle  = InterleavedGradientNoise(float2(iPos));
	float Offset = (1.0/4.0)  *  (( iPos.y - iPos.x)&3);
	return float2(Angle, Offset);
}

float3 GetRandomVector(uint2 iPos )
{
	iPos.y = 16384-iPos.y;

	float3 RandomVec	= float3(0,0,0);
	float3 RandomTexVec	= float3(0,0,0);
	float ScaleOffset;

	float TemporalCos = GTAOParams[0].x;
	float TemporalSin = GTAOParams[0].y;

	float GradientNoise = InterleavedGradientNoise(float2(iPos));

	RandomTexVec.x = cos((GradientNoise*PI) );
	RandomTexVec.y = sin((GradientNoise*PI) );

	ScaleOffset = (1.0/4.0)  *  (( iPos.y - iPos.x) & 3);
//	ScaleOffset = (1.0/5.0)  *  (( iPos.y - iPos.x) % 5);

	RandomVec.x = dot(RandomTexVec.xy, float2(TemporalCos, -TemporalSin ));
    RandomVec.y = dot(RandomTexVec.xy, float2(TemporalSin, TemporalCos ));
	RandomVec.z = frac(ScaleOffset + GTAOParams[0].z);

	return RandomVec;
}



/*
*
* HORIZON SEARCH AND INNER INTEGRATE COMBINED
*
*/
void GTAOCombinedPSandCS(in float2 UV, in uint2 iPos, out float OutColor)
{
	OutColor = 0;

	// Offset by a fraction of a pixel to unsure we don't hit between pixels when running at half res
	float2 QuarterOffset = AOSceneViewport_ExtentInverse * 0.125;
	float2 TexUV = UV + QuarterOffset;
	UV += QuarterOffset;


	float DeviceZ		 = LookupDeviceZ(TexUV );
	float SceneDepth	 = ConvertFromDeviceZ(DeviceZ);

	if(SceneDepth > ScreenSpaceAOParams[4].w)	
	{		
		OutColor = 1;
		return;
	}
	
	float3 ViewSpacePos		= ScreenToViewPos(TexUV,SceneDepth);
	float3 ViewSpaceNormal	= GetNormal(TexUV, ViewSpacePos);
	float3 ViewDir			= normalize(-ViewSpacePos.xyz);

	const float WorldRadius = GTAOParams[3].y;	

	float InvTanHalfFov				= ScreenSpaceAOParams[3].w;
	float FOVScale					= AOSceneViewport_Extent.y * InvTanHalfFov; // TODO

	// Get Radius in ScreenSpace (in pixels)
	float WorldRadiusAdj		= WorldRadius * FOVScale;	
	float PixelRadius			= max( min( WorldRadiusAdj / ViewSpacePos.z, GTAO_MAX_PIXEL_SCREEN_RADIUS ), (float) GTAO_NUMTAPS );
	float StepRadius			= PixelRadius / ( (float) GTAO_NUMTAPS + 1 );
	float AttenFactor			= 2.0 / (WorldRadius * WorldRadius);

	// Get the randomized Direction to sample and the step offset
	float3 RandomAndOffset		= GetRandomVector(iPos);
	float2 RandomVec			= RandomAndOffset.xy;
	float  Offset				= RandomAndOffset.z;

	float Sum=0.0;

	uint NumAngles			= (uint) GTAOParams[4].y;
	float SinDeltaAngle		= GTAOParams[4].z;
	float CosDeltaAngle		= GTAOParams[4].w;

	float2 ScreenDir = float2(RandomVec.x, RandomVec.y);

	for(uint Angle =0; Angle < NumAngles; Angle++)
	{
		float2 Angles = SearchForLargestAngleDual_HZB(GTAO_NUMTAPS, TexUV, ScreenDir* View.BufferSizeAndInvSize.zw, StepRadius, 
												        Offset, ViewSpacePos, ViewDir, AttenFactor);

		Sum += ComputeInnerIntegral(TexUV,  Angles, ScreenDir, ViewDir, ViewSpaceNormal, SceneDepth);

		// Rotate for the next angle
		float2 TempScreenDir = ScreenDir.xy;
		ScreenDir.x = (TempScreenDir.x *  CosDeltaAngle) + (TempScreenDir.y * -SinDeltaAngle);
		ScreenDir.y = (TempScreenDir.x *  SinDeltaAngle) + (TempScreenDir.y * CosDeltaAngle);
		Offset      = frac(Offset + 0.617);
	}

	float AO = Sum;

	AO = AO   / ((float)NumAngles);
	AO *= 2.0/PI;


	// Fade out based on user defined distance
	float Mul = ScreenSpaceAOParams[4].x;
	float Add = ScreenSpaceAOParams[4].y;
	AO = lerp(AO, 1, saturate(SceneDepth * Mul + Add));




	OutColor	= AO ;
	return;
}


void GTAOCombinedPS(in float4 UVAndScreenPos : TEXCOORD0, out float OutColor : SV_Target0)
{
	int2 iPos = int2( UVAndScreenPos.xy * AOViewport_Extent );
	GTAOCombinedPSandCS(UVAndScreenPos.xy, iPos, OutColor);	
}


#if COMPUTE_SHADER
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void GTAOCombinedCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
	float OutColor		= 0;

	int2   PixelPos		= DispatchThreadId + AOViewport_ViewportMin; 
	float2 PixelCenter	= (float2)PixelPos + float2(0.5, 0.5);
	float2 BufferUV		= PixelCenter.xy * GTAOParams[2].zw;

	GTAOCombinedPSandCS(BufferUV, PixelPos, OutColor);

	OutTexture[PixelPos]		= OutColor;
}
#endif




/*
*
*	INNER INTEGRATE
*
*/

Texture2D		HorizonsTexture;
SamplerState	HorizonsTextureSampler;


float GTAOInnerIntegratePSandCS(in float2 UV, in uint2 iPos)
{
	float2 QuarterOffset = AOSceneViewport_ExtentInverse * 0.125;
	UV += QuarterOffset;

	// Read the angles buffer
	float SceneDepth	 = GetDepthFromAOInput(UV);		
	if(SceneDepth > ScreenSpaceAOParams[4].w)	
	{		
		return 1;
	}

	float4 Angles			= Texture2DSample(HorizonsTexture, HorizonsTextureSampler, UV);	// Angles computed from previous pass
	Angles				    = Angles * PI;

	// Get Angle
	float2 RandomVec		= GetRandomVector(iPos).xy;
	float2 ScreenDir		= float2(RandomVec.x, RandomVec.y);

	// ViewspacePos and Normal
	float3 ViewSpacePos		= ScreenToViewPos(UV, SceneDepth);
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(iPos, 0)));
	float3 WorldNormal = TopLayerData.WorldNormal;
#else
	float3 WorldNormal		= GetGBufferData(UV, false).WorldNormal;
#endif
	float3 ViewSpaceNormal	= normalize(mul(WorldNormal, (float3x3)View.TranslatedWorldToView));
	float3 ViewDir			= -normalize(ViewSpacePos.xyz);	// TODO - This is a function of UV only.

	float AO = ComputeInnerIntegral(UV, Angles.xy, ScreenDir, ViewDir, ViewSpaceNormal, SceneDepth);

	uint NumAngles			= (uint) GTAOParams[4].y;
	if(NumAngles>1)
	{
		ScreenDir.xy = float2(-ScreenDir.y, ScreenDir.x);
		AO += ComputeInnerIntegral(UV, Angles.zw, ScreenDir, ViewDir, ViewSpaceNormal, SceneDepth);
		AO *=0.5;
	}

	AO *= 2.0/PI;

	// Fade out based on user defined distance
	float Mul = ScreenSpaceAOParams[4].x;
	float Add = ScreenSpaceAOParams[4].y;
	AO = lerp(AO, 1, saturate(SceneDepth * Mul + Add));

	return AO ;
}



void GTAOInnerIntegratePS(in noperspective float4 UVAndScreenPos : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	int2 iPos = int2( UVAndScreenPos.xy * AOViewport_Extent );
	float AO = 	GTAOInnerIntegratePSandCS(UVAndScreenPos.xy, iPos);	
	OutColor = AO;
}


#if COMPUTE_SHADER
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void GTAOInnerIntegrateCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{

	float  OutColor		= 0;
	int2   PixelPos		= DispatchThreadId + AOViewport_ViewportMin; 
	float2 PixelCenter	= (float2)PixelPos + float2(0.5, 0.5);
	float2 BufferUV		= PixelCenter.xy * GTAOParams[2].zw;

	float AO = GTAOInnerIntegratePSandCS(BufferUV,PixelPos);
	
	OutTexture[PixelPos] = AO;
}
#endif






/*
*
* HORIZON SEARCH ONLY
*
*/

float4 HorizonSearchPSandCS(in float2 UV,  in uint2 iPos)
{
	float4 OutHorizons = 0;

	// Offset by a fraction of a pixel to unsure we don't hit between pixels when running at half res
	float2 QuarterOffset = AOSceneViewport_ExtentInverse * 0.125;
	UV = UV + QuarterOffset;

	float DeviceZ		 = LookupDeviceZ(UV );
	float SceneDepth	 = ConvertFromDeviceZ(DeviceZ);

	if(SceneDepth > ScreenSpaceAOParams[4].w)	
	{		
		OutHorizons = 0;
		return OutHorizons;
	}
	
	float3 ViewSpacePos		= ScreenToViewPos(UV,SceneDepth);
	float3 ViewSpaceNormal	= GetNormal(UV, ViewSpacePos);
	float3 ViewDir			= normalize(-ViewSpacePos.xyz);

	const float WorldRadius = GTAOParams[3].y;	
	float InvTanHalfFov				= ScreenSpaceAOParams[3].w;
	float FOVScale					= AOSceneViewport_Extent.y * InvTanHalfFov; 

	// Get Radius in ScreenSpace (in pixels)
	float WorldRadiusAdj		= WorldRadius * FOVScale;	
	float PixelRadius			= max( min( WorldRadiusAdj / ViewSpacePos.z, GTAO_MAX_PIXEL_SCREEN_RADIUS ), (float) GTAO_NUMTAPS );
	float StepRadius			= PixelRadius / ( (float) GTAO_NUMTAPS + 1 );
	float AttenFactor			= 2.0 / (WorldRadius * WorldRadius);

	// Get the randomized Direction to sample and the step offset
	float3 RandomAndOffset		= GetRandomVector(iPos);
	float2 RandomVec			= RandomAndOffset.xy;
	float  Offset				= RandomAndOffset.z;

	float Sum=0.0;

	uint NumAngles			= (uint) GTAOParams[4].y;
	float SinDeltaAngle		= GTAOParams[4].z;
	float CosDeltaAngle		= GTAOParams[4].w;

	float2 ScreenDir = float2(RandomVec.x, RandomVec.y);

	// First Angle
	float2 Angles = SearchForLargestAngleDual_HZB(GTAO_NUMTAPS, UV, ScreenDir* View.BufferSizeAndInvSize.zw, StepRadius, 
 									        Offset, ViewSpacePos, ViewDir, AttenFactor);

	Angles /= PI;

	float2 Angles2=0;
	if(NumAngles>1)
	{
		// Rotate for the next angle
		float2 TempScreenDir = ScreenDir.xy;
		ScreenDir.x = (TempScreenDir.x *  CosDeltaAngle) + (TempScreenDir.y * -SinDeltaAngle);
		ScreenDir.y = (TempScreenDir.x *  SinDeltaAngle) + (TempScreenDir.y * CosDeltaAngle);

		Angles2 = SearchForLargestAngleDual_HZB(GTAO_NUMTAPS, UV, ScreenDir* View.BufferSizeAndInvSize.zw, StepRadius, 
									        Offset, ViewSpacePos, ViewDir, AttenFactor);
		Angles2 /= PI;
	}

	OutHorizons.xy = Angles;
	OutHorizons.zw = Angles2;

	return OutHorizons;
}


void HorizonSearchPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	int2 iPos = int2( UVAndScreenPos.xy * AOViewport_Extent );
	OutColor = HorizonSearchPSandCS(UVAndScreenPos.xy, iPos);	
}


#if COMPUTE_SHADER
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void HorizonSearchCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
	float2 OutColor		= 0;
	int2   PixelPos		= DispatchThreadId + AOViewport_ViewportMin; 
	float2 PixelCenter	= (float2)PixelPos + float2(0.5, 0.5);
	float2 BufferUV		= PixelCenter.xy * GTAOParams[2].zw;

	float4 Horizons = HorizonSearchPSandCS(BufferUV,PixelPos);
	
	HorizonOutTexture[PixelPos] = Horizons;
}
#endif

/*
 *
 *    TEMPORAL FILTER
 *
 */

Texture2D SceneVelocityTexture;
SamplerState SceneVelocityTextureSampler;
float4 BlendParams;
float3 ReprojectPos(float2 UV, float Depth)
{
	// Given a UV reproject where this was in the previous frame
	// Camera motion for pixel (in ScreenPos space).
	float2 ThisScreen = (UV.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	float4 ThisClip = float4( ThisScreen, Depth, 1 );
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float2 PrevScreen = PrevClip.xy / PrevClip.w;

	float4 EncodedVelocity = Texture2DSampleLevel(SceneVelocityTexture, SceneVelocityTextureSampler, UV,0);
	if( EncodedVelocity.x > 0.0 )
	{
		PrevScreen = ThisClip.xy - DecodeVelocityFromTexture(EncodedVelocity).xy;
	}
	
	float2 PrevUV = PrevScreen.xy * PrevScreenPositionScaleBias.xy + PrevScreenPositionScaleBias.zw;
	return float3(PrevUV, PrevClip.z/ PrevClip.w);
}

float ReadHistoryClamp(float2 UV, float MinAO, float MaxAO)
{
	float	BilinearWeights[4];

	float2 PixUV   = (UV * HistoryTextureSize)-0.5;
	float2 FloorUV = floor(PixUV);
	float2 FracUV  = (PixUV - FloorUV); 
	UV			   = (FloorUV * HistoryTexturePixelSize) + (HistoryTexturePixelSize*0.5);

	BilinearWeights[0] = (1.0 -	FracUV.x) * ( 1.0 -	FracUV.y);
	BilinearWeights[1] = (		FracUV.x) * ( 1.0 -	FracUV.y);
	BilinearWeights[2] = (1.0 -	FracUV.x) * (       FracUV.y);
	BilinearWeights[3] = (		FracUV.x) * (       FracUV.y);

	// Read the 4 previous depths and History
	float HistoryAO[4];

	float2 dUV = HistoryTexturePixelSize;

	// TODO - Use GatherR when available
	HistoryAO[0] = Texture2DSample(HistoryTexture, HistoryTextureSampler, UV + float2(	  0,     0)).r;
	HistoryAO[1] = Texture2DSample(HistoryTexture, HistoryTextureSampler, UV + float2(dUV.x,     0)).r;
	HistoryAO[2] = Texture2DSample(HistoryTexture, HistoryTextureSampler, UV + float2(    0, dUV.y)).r;
	HistoryAO[3] = Texture2DSample(HistoryTexture, HistoryTextureSampler, UV + float2(dUV.x, dUV.y)).r;

	float VisHistory = 0;
	for(int i=0; i<4; i++)
	{
		HistoryAO[i] = clamp(HistoryAO[i], MinAO, MaxAO);
		VisHistory += BilinearWeights[i] * HistoryAO[i];
	}


	return VisHistory;
}

Texture2D GTAOTemporalInput;
SamplerState GTAOTemporalSampler;
float2 GTAOTemporalInputPixelSize;

void NeighbourhoodClamp(float2 UV, float BaseAO, inout float MinAO, inout float MaxAO)
{
	float2 dUV = GTAOTemporalInputPixelSize * 1.5;

	#define NumSamples 4

	float AONeighbours[NumSamples];

	AONeighbours[0] = Texture2DSample(GTAOTemporalInput, GTAOTemporalSampler, UV + float2(-dUV.x,-dUV.y) ).r;
	AONeighbours[1] = Texture2DSample(GTAOTemporalInput, GTAOTemporalSampler, UV + float2(-dUV.x, dUV.y) ).r;
	AONeighbours[2] = Texture2DSample(GTAOTemporalInput, GTAOTemporalSampler, UV + float2( dUV.x,-dUV.y) ).r;
	AONeighbours[3] = Texture2DSample(GTAOTemporalInput, GTAOTemporalSampler, UV + float2( dUV.x, dUV.y) ).r;


#if GTAO_VARIANCE_CLIPPING
	float AOAverage = 0;
	float AOSquared = 0;

	for(int i=0; i<4; i++)
	{
		AOAverage += AONeighbours[i];
		AOSquared += AONeighbours[i]*AONeighbours[i];
	}

	float Mu = AOAverage / NumSamples;
	float Sigma = sqrt(AOSquared / NumSamples -  (Mu*Mu));
	
	MinAO  = max( Mu - Sigma * 0.8, 0.0 );
	MaxAO  = min( Mu + Sigma * 0.8, 1.0 );
#else
	MinAO = min(BaseAO, min(min(AONeighbours[0], AONeighbours[1]), min(AONeighbours[2], AONeighbours[3])));
	MaxAO = max(BaseAO, max(max(AONeighbours[0], AONeighbours[1]), max(AONeighbours[2], AONeighbours[3])));
#endif
}


float CompareVeloc(float2 V1, float2 V2)
{
	float2 V12 = V1-V2;
	return 1-saturate( abs(V12.x + V12.y) * 100);

}

void GTAOTemporalFilterPSAndCS(float2 UV, inout float OutAO)
{
	float BlendWeight = GTAOParams[4].x;

	float2 QuarterOffset = AOSceneViewport_ExtentInverse * 0.125;
	UV = UV + QuarterOffset;

	// Latest AO value
	float NewAO					= Texture2DSample(GTAOTemporalInput, GTAOTemporalSampler, UV).r;

	// Current depth of the rendered Scene
	float CurrDepthDeviceZ		= Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV).r;
	float CurrDepth				= ConvertFromDeviceZ( CurrDepthDeviceZ);

	// Previous UV value
	float3 PrevUVDepth			= ReprojectPos( UV,  CurrDepthDeviceZ);
	float  CurrDepthReproject	= ConvertFromDeviceZ(PrevUVDepth.z);
	float2 PrevUV				= PrevUVDepth.xy;

	float2 PixVelocity			= UV - PrevUV;
	float  VelocityMag			= saturate(length(PixVelocity)*100);
	
	// Compare velocities 
	float2 DestVeloc=0;
	{
		float DestDeviceZ		= Texture2DSample(ZCurrTexture, ZCurrTextureSampler, PrevUVDepth.xy).r;
		float3 Reproj			= ReprojectPos( PrevUVDepth.xy,  DestDeviceZ); 
		DestVeloc				= PrevUVDepth.xy - Reproj.xy;
	}

	float VelocCompare			= CompareVeloc(PixVelocity, DestVeloc);

	// Get an acceptable range of values we care about from the current AO
	float RangeVal	   = lerp(0.1, 0.00, VelocityMag);
	float MinAO = saturate(NewAO - RangeVal);
	float MaxAO = saturate(NewAO + RangeVal);

	// Simple history value
	float HistoryPrevUV				= ReadHistoryClamp(PrevUV, MinAO, MaxAO);
	float HistoryThisUV				= Texture2DSample(HistoryTexture, HistoryTextureSampler, UV ).r;
	HistoryThisUV = clamp(HistoryThisUV, MinAO, MaxAO);

	float HistoryAO = HistoryPrevUV;

	HistoryAO = lerp(HistoryThisUV, HistoryPrevUV, VelocCompare);

	OutAO				= lerp(HistoryAO, NewAO, BlendWeight);
}

void GTAOTemporalFilterPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float OutAO = 0;
	
	GTAOTemporalFilterPSAndCS(UVAndScreenPos.xy, OutAO);
	OutColor	= OutAO;
}


#if COMPUTE_SHADER
[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void GTAOTemporalFilterCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
	float OutColor		= 0;

	int2   PixelPos		= DispatchThreadId + AOViewport_ViewportMin; 
	float2 PixelCenter	= (float2)PixelPos + float2(0.5, 0.5);
	float2 BufferUV		= PixelCenter.xy * AOViewport_ExtentInverse;

	GTAOTemporalFilterPSAndCS(BufferUV, OutColor);

	OutTexture[PixelPos]			= OutColor;
}

#endif


#if COMPUTE_SHADER

Texture2D GTAOSpatialFilterTexture;
Texture2D GTAOSpatialFilterDepthTexture;
uint2 GTAOSpatialFilterExtents;

float4 GTAOSpatialFilterParams;
float4 GTAOSpatialFilterWidth;


// The 5x5 filter works on a threadgroup of size 16x8 (128 pixels)
// We need to read in the 16x8 and a 2 pixel border around. So this is 20x12 (240 pixels)
// Each thread reads in 2 pixels each

// We make the array 32 wide so it plays better with bank conflicts
#define LDS_WIDTH 20 

groupshared float AOData[ LDS_WIDTH*12]; 
groupshared float ZData[  LDS_WIDTH*12];

int GetLDSLocation(int x, int y)
{
	x+=2; y+=2;
	return ((y*LDS_WIDTH) + x) ;
}

float GetAOLin(int loc)
{
	return AOData [loc];
}

float GetZLin(int loc)
{
	return ZData[loc];
}



float GetAO(int x, int y)
{
	x+=2; y+=2;
	return AOData [(y*LDS_WIDTH) + x];
}

float GetZ(int x, int y)
{
	x+=2; y+=2;
	return ZData [(y*LDS_WIDTH) + x];
}


[numthreads(16, 8, 1)]
void GTAOSpatialFilterCS(
	int   GroupIndex: SV_GroupIndex,
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
	int2 GTId = int2(GroupThreadId);

	
	// Position on the screen We care about
	int2   PixelPos				= DispatchThreadId + AOViewport_ViewportMin; 

	// Firstly get the origin in the screen of the 16x8 inner box
	int2   FullGroupOrigin		= int2(GroupId.x * 16, GroupId.y * 8) + AOViewport_ViewportMin; 
	int2   FullGroupOriginM2	= FullGroupOrigin.xy - int2(2,2);

	uint pixIdx = (GroupIndex*2);

	float DownsampleFactor =  GTAOSpatialFilterParams.x;

	// Downsampled version. Note that the Z is double the res of the Z
	if(pixIdx < (20*12) )
	{
		uint XPos = pixIdx%20;
		uint YPos = pixIdx/20;

		int LDSPos = (YPos*LDS_WIDTH) + XPos;

		int2 ReadXYAO = FullGroupOriginM2 + int2(XPos,YPos);
		int2 ReadXYZ  = ReadXYAO*DownsampleFactor;

		float AO = GTAOSpatialFilterTexture.Load(int3(ReadXYAO, 0)).r;
		float  Z = GTAOSpatialFilterDepthTexture.Load(     int3(ReadXYZ, 0)).r;
		AOData[ LDSPos ] = AO;
		ZData[  LDSPos ] = Z;

		// Next pixel
		LDSPos++;

		ReadXYAO.x	+=1;
		ReadXYZ.x	+=DownsampleFactor;

		AO = GTAOSpatialFilterTexture.Load(int3(ReadXYAO, 0)).r;
		Z  = GTAOSpatialFilterDepthTexture.Load(     int3(ReadXYZ, 0)).r;
		AOData[ LDSPos ] = AO;
		ZData[  LDSPos ] = Z;
	}
	GroupMemoryBarrierWithGroupSync();

	// Get the differences in Z at this pixel. This is needed for the bilateral filter
	float ThisZ		= GetZ(GTId.x, GTId.y);
	float ThisZLin	=ConvertFromDeviceZ( ThisZ);

	float2 ZDiff;

	int FilterMin = int(GTAOSpatialFilterWidth.x);
	int FilterMax = int(GTAOSpatialFilterWidth.y);
	int LDSBase = GetLDSLocation(GTId.x + FilterMin, GTId.y + FilterMin);

	//Get X Delta
	int LDSCentre = GetLDSLocation(GTId.x , GTId.y);
	{

		float XM2Z	= GetZLin(LDSCentre-2);
		float XM1Z	= GetZLin(LDSCentre-1);
		float XP1Z	= GetZLin(LDSCentre+1);
		float XP2Z	= GetZLin(LDSCentre+2);

		// Get extrapolated point either side
		float C1 = abs((XM1Z + (XM1Z - XM2Z)) - ThisZ);
		float C2 = abs((XP1Z + (XP1Z - XP2Z)) - ThisZ);

		if(C1 < C2)
		{
			ZDiff.x = XM1Z - XM2Z;
		}
		else
		{
			ZDiff.x = XP2Z - XP1Z;
		}
	}


	//Get Y Delta
	{
		float YM2Z	= GetZLin(LDSCentre-(2*LDS_WIDTH));
		float YM1Z	= GetZLin(LDSCentre-(1*LDS_WIDTH));
		float YP1Z	= GetZLin(LDSCentre+(1*LDS_WIDTH));
		float YP2Z	= GetZLin(LDSCentre+(2*LDS_WIDTH));
										 
		// Get extrapolated point either side
		float C1 = abs((YM1Z + (YM1Z - YM2Z)) - ThisZ);
		float C2 = abs((YP1Z + (YP1Z - YP2Z)) - ThisZ);

		if(C1 < C2)
		{
			ZDiff.y = YM1Z - YM2Z;
		}
		else
		{
			ZDiff.y = YP2Z - YP1Z;
		}
	}

	// Do the blur
	float SumAO		= 0;
	float SumWeight = 0;

	int x,y;

	// Get the Z Value to compare against 

	float DepthBase = ThisZ  +(ZDiff.x*FilterMin) + (ZDiff.y*FilterMin);

	float SimpleBlur=0.0;

	for(y=FilterMin; y<=FilterMax; y++)
	{
		float PlaneZ = DepthBase;

		int LDSLineBase = LDSBase;
		LDSBase += LDS_WIDTH;


		for(x=FilterMin; x<=FilterMax; x++)
		{
			float Sample_AO = GetAOLin(LDSLineBase);
			float SampleZ   = GetZLin( LDSLineBase);
			LDSLineBase++;


			// Get the bilateral weight. This is a function of the difference in height between the plane equation and the base depth
			// Compare the Z at this sample with the gradients 
			float SampleZDiff = abs(PlaneZ - SampleZ);
			
			const float SpatialFilterWeight = 20000;
			float Weight =  1.0f  - saturate(SampleZDiff*SpatialFilterWeight );


			SumAO += Sample_AO * Weight;
			//SimpleBlur += Sample_AO;
			SumWeight += Weight;
		
			PlaneZ += ZDiff.x;
		}
		DepthBase += ZDiff.y;
	}
	SumAO /=SumWeight;

	SumAO *= (PI/2.0) ;

	// user adjust AO
	float AmbientOcclusionIntensity = ScreenSpaceAOParams[0].w;
	float AmbientOcclusionPower		= ScreenSpaceAOParams[0].x*0.5;
	SumAO = 1 - (1 - pow(abs(SumAO), AmbientOcclusionPower)) * AmbientOcclusionIntensity;


	OutTexture[PixelPos] = SumAO;
}
#endif



float2 SpatialDiff;

// Single axis blur filter for Pixel Shaders
void GTAOSpatialFilterPS(float4 UVAndScreenPos : TEXCOORD0, 
						 float4 SvPosition : SV_POSITION, 
						 out float4 OutColor : SV_Target0) 
{
	float2 UV = UVAndScreenPos.xy;

	// Do a 3 pixel wide spatial filter
	float OutAO = 0;

	float2 Offset = PostprocessInput0Size.zw;
	float2 Offset2 = Offset*2;

	// Get Depth and AO at this pixel
	float AO_C					= Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, UV).r;
	float Z_C					= ConvertFromDeviceZ( Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV).r);

	float AO_M1					= Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, UV - Offset).r;
	float Z_M1					= ConvertFromDeviceZ( Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV- Offset).r);

	float AO_P1					= Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, UV + Offset).r;
	float Z_P1					= ConvertFromDeviceZ( Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV + Offset).r);

	float AO_M2					= Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, UV - Offset2).r;
	float Z_M2					= ConvertFromDeviceZ( Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV- Offset).r);

	float AO_P2					= Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, UV + Offset2).r;
	float Z_P2					= ConvertFromDeviceZ( Texture2DSample(ZCurrTexture, ZCurrTextureSampler, UV + Offset).r);


	float DiffZ = min( abs(Z_C - Z_M1), abs(Z_C - Z_P1) );


	const float SpatialFilterWeight = 1000;
	float SampleZDiff=0;
	float Weight=0;

	// Blend the values
	float SumWeight = 1.0;
	float TotalAO   = AO_C;

	// Minus 2
	SampleZDiff = abs(Z_C - Z_M2);
	SampleZDiff -= DiffZ*2;
	Weight = 1.0f - saturate(SampleZDiff*SpatialFilterWeight );
	TotalAO += AO_M2 * Weight;
	SumWeight += Weight;


	// Minus 2
	SampleZDiff = abs(Z_C - Z_M1);
	SampleZDiff -= DiffZ;
	Weight = 1.0f - saturate(SampleZDiff*SpatialFilterWeight );
	TotalAO += AO_M1 * Weight;
	SumWeight += Weight;


	// Plus 2
	SampleZDiff = abs(Z_C - Z_P2);
	SampleZDiff -= DiffZ*2;
	Weight = 1.0f - saturate(SampleZDiff*SpatialFilterWeight );
	TotalAO += AO_P2 * Weight;
	SumWeight += Weight;


	// Plus 1
	SampleZDiff = abs(Z_C - Z_P1);
	SampleZDiff -= DiffZ;
	Weight = 1.0f - saturate(SampleZDiff*SpatialFilterWeight );
	TotalAO += AO_P1 * Weight;
	SumWeight += Weight;


	TotalAO /= SumWeight;

	// Blend them together based on depth
	OutColor	= AO_C;
}








#if COMPUTE_SHADER
#define UPSAMPLE_LDS_WIDTH 20 
groupshared float FullZData[  (16+1) * UPSAMPLE_LDS_WIDTH  ];
groupshared float LowAOData[  9*10];

float GetBlendAO(float AO1, float AO2, float Z1, float Z2, float ZMid)
{
	float dZ		= Z2 - Z1;

	float Epsilon = 0.00001f;
	if(abs(dZ) < Epsilon)
	{
		return (AO1+AO2) * 0.5;
	}

	float Ratio = saturate((ZMid -Z1) * (1.0/dZ) );
	return  (AO1 * (1.0-Ratio)) + ( AO2 * Ratio); 
}

[numthreads(8, 8, 1)]
void SmartUpsample(
	int   GroupIndex: SV_GroupIndex,
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID) 
{
#if 0
	int2 GTId = int2(GroupThreadId);
	// Get the pixel Pos of the final position

	int2   PixelPos				= DispatchThreadId*2 + ScreenSpaceAOParams[5].zw; 
	// Each thread will compute 4 output colours . We need a 1 pixel border around the depth buffer so each thread will read in 5 pixel into the 17x17 buffer
	int2   FullGroupOrigin		= int2(GroupId.x * THREADGROUP_SIZEX, GroupId.y * THREADGROUP_SIZEY) + ScreenSpaceAOParams[5].zw; 
	uint2  TileOrigin = GroupId.xy *16;
	
	// Read in 4 pixels
	uint2 PixelPosInTile  = GroupThreadId.xy *2;
	uint  FullZLDSOffset = (PixelPosInTile.y *UPSAMPLE_LDS_WIDTH) + PixelPosInTile.x;

	SetAOVal(PostprocessInput0.Load(int3(FullGroupOriginM2 + int2(XPos,YPos), 0)).r, XPos, YPos);
	SetZVal( ZReadTexture.Load(     int3(FullGroupOriginM2 + int2(XPos,YPos), 0)).r, XPos, YPos);
	float BotLeftZ  = ( ZReadTexture.Load(  int3(TileOrigin + PixelPosInTile +uint2(1,0), 0)).r );
	SetAOVal(PostprocessInput0.Load(int3(FullGroupOriginM2 + int2(XPos,YPos), 0)).r, XPos, YPos);
	SetZVal( ZReadTexture.Load(     int3(FullGroupOriginM2 + int2(XPos,YPos), 0)).r, XPos, YPos);

	FullZData[FullZLDSOffset]						= TopLeftZ  ;
	FullZData[FullZLDSOffset+1]						= TopRightZ ;
	FullZData[FullZLDSOffset+UPSAMPLE_LDS_WIDTH]	= BotLeftZ  ;
	FullZData[FullZLDSOffset+UPSAMPLE_LDS_WIDTH+1]	= BotRightZ ;

	// The final pixel needs to be the border (17+16 == 33 of them)
	uint2 BorderXY;
	uint  BorderLDSOffset =0;
	if(GroupIndex < 17)
	if (any(DispatchThreadId >= (uint2)SpatialFilterParams.zw))
	{
		BorderXY		= uint2(16,GroupIndex);
		BorderLDSOffset = 16 + (GroupIndex*UPSAMPLE_LDS_WIDTH);
	}
	else
	{
		BorderXY	    = uint2(GroupIndex-17, 16);
		BorderLDSOffset = (GroupIndex-17) + (UPSAMPLE_LDS_WIDTH*16);
	}

	if(GroupIndex < 33)
	{
		FullZData[BorderLDSOffset] =  ( ZReadTexture.Load(  int3(TileOrigin + BorderXY, 0)).r );
	}


	GroupMemoryBarrierWithGroupSync();


// Now read in the Color data which is 1/4 res
	uint2 LowTileOrigin = GroupId.xy *8;
	uint2 LowPixelPosInTile  = GroupThreadId.xy;

	uint LowAOLDSOffset = (LowPixelPosInTile.y *9) + LowPixelPosInTile.x;
	float ThisAO  =  PostprocessInput0.Load(int3(LowTileOrigin + LowPixelPosInTile , 0)).r;
	LowAOData[LowAOLDSOffset] = ThisAO;

	// Read in the border
	if(GroupIndex < 9)
	{
		BorderXY = uint2(8,GroupIndex);
		BorderLDSOffset = 8 + (GroupIndex*9);
	}
	else
	{
		BorderXY = uint2(GroupIndex-9, 8);
		BorderLDSOffset = (GroupIndex-9) + (9*8);
	}
	GroupMemoryBarrierWithGroupSync();
	
	if(GroupIndex < 17)
		LowAOData[BorderLDSOffset] = PostprocessInput0.Load(int3(LowTileOrigin + BorderXY , 0)).r;


	GroupMemoryBarrierWithGroupSync();

	// All Data read we can now Process the 4 AO Values
	float FinalAO_TL;
	float FinalAO_TR;
	float FinalAO_BL;
	float FinalAO_BR;


// Top Left - Easy this is the same as the low res colour read in
	FinalAO_TL = ThisAO;


// Top Right - This is a weighted blend of the Top Left and the pixel to the right
	float Right_AO	= LowAOData[LowAOLDSOffset+1];
	float Ext_Z		= FullZData[FullZLDSOffset+2];
	FinalAO_TR		= GetBlendAO(ThisAO, Right_AO, TopLeftZ, Ext_Z,  TopRightZ);


// Bottom Left - This is a weighted blend of the Top Left and the pixel below
	float Bottom_AO = LowAOData[LowAOLDSOffset+9];
	Ext_Z		    = FullZData[FullZLDSOffset+(2*UPSAMPLE_LDS_WIDTH)];
	FinalAO_BL		= GetBlendAO(ThisAO, Bottom_AO, TopLeftZ, Ext_Z,  BotLeftZ);

	
// Bottom Right - This is a weighted blend of the Top Left and the pixel to the bottom right
	float BotRight_AO = LowAOData[LowAOLDSOffset+9+1];
	Ext_Z		      = FullZData[FullZLDSOffset+(2*UPSAMPLE_LDS_WIDTH)+2];
	FinalAO_BR		  = GetBlendAO(ThisAO, BotRight_AO, TopLeftZ, Ext_Z,  BotRightZ);

	
	OutTexture[PixelPos + uint2(0,0) ] = FinalAO_TL ;
	OutTexture[PixelPos + uint2(1,0) ] = FinalAO_TR ;
	OutTexture[PixelPos + uint2(0,1) ] = FinalAO_BL ;
	OutTexture[PixelPos + uint2(1,1) ] = FinalAO_BR ;
	#endif

}

#endif

#endif // UPSAMPLE_FILTER_SHADER