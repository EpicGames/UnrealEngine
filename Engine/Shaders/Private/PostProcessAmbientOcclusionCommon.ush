// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostprocessAmbientOcclusion.ush: Common code (mostly SSAO) used by 
	PostProcessAmbientOcclusion.usf and PostProcessAmbientOcclusionMobile.usf
=============================================================================*/

// [0]: .x:AmbientOcclusionPower, .y:AmbientOcclusionBias/BiasDistance, .z:1/AmbientOcclusionDistance, .w:AmbientOcclusionIntensity
// [1]: .xy:ViewportUVToRandomUV, .z:AORadiusInShader, .w:Ratio
// [2]: .x:ScaleFactor(e.g. 4 if current RT is a quarter in size), .y:InvThreshold, .z:ScaleRadiusInWorldSpace(0:VS/1:WS), .w:MipBlend
// [3]: .xy:TemporalAARandomOffset, .z:StaticFraction, .w: InvTanHalfFov
// [4]: .x:Multipler for FadeDistance/Radius, .y:Additive for FadeDistance/Radius, .z:clamped HzbStepMipLevelFactorValue .w: unused
float4 ScreenSpaceAOParams[5];

SCREEN_PASS_TEXTURE_VIEWPORT(AOViewport)
SCREEN_PASS_TEXTURE_VIEWPORT(AOSceneViewport)

// needed to prevent AO seam near 16 bit float maximum, this feactor pushed the problem far out and it seems to not have a visual degradion nearby
const static float Constant_Float16F_Scale =  4096.0f * 32.0f;

// only for MainSetupPS()
float ThresholdInverse;
float2 InputExtentInverse;

/** RGBA8 linear texture containing random normals */
Texture2D RandomNormalTexture;
SamplerState RandomNormalTextureSampler;

//----------------------------------------------------------------------------------------------------------------------

#include "HZB.ush"

float GetHZBDepth(float2 ScreenPos, float MipLevel)
{
	float2 HZBUV = ApplyScreenTransform(ScreenPos, ScreenPosToHZBUVScaleBias);
	float HZBDepth = Texture2DSampleLevel( HZBTexture, HZBSampler, HZBUV, MipLevel).r;
	return ConvertFromDeviceZ(HZBDepth);
}

//----------------------------------------------------------------------------------------------------------------------

Texture2D SSAO_SetupTexture;
Texture2D SSAO_NormalsTexture;
Texture2D SSAO_DownsampledAO;
SamplerState SSAO_Sampler;

float2 SSAO_DownsampledAOInverseSize;
float2 SSAO_DownsampledAOUVViewportMin;
float2 SSAO_DownsampledAOUVViewportMax;
float2 SSAO_SvPositionScaleBias;
uint SSAO_ShadingModelsExcludeMask;

// could be moved to a more central spot
// @param ScreenPos -1 .. 1
float3 ReconstructCSPos(float SceneDepth, float2 ScreenPos)
{
	return float3(ScreenPos * SceneDepth, SceneDepth);
}

// 0: not similar .. 1:very similar
float ComputeDepthSimilarity(float DepthA, float DepthB, float TweakScale)
{
	return saturate(1 - abs(DepthA - DepthB) * TweakScale);
}

float TakeSmallerAbsDelta(float left, float mid, float right)
{
	float a = mid - left;
	float b = right - mid;

	return (abs(a) < abs(b)) ? a : b;
}

// could use ddx,ddy but that would have less quality and would nto work fo ComputeShaders
// @return not normalized normal in world space
float3 ReconstructNormalFromDepthBuffer(float4 SvPosition)
{
	// could use a modified version of GatherSceneDepth later on
	float DeviceZ = LookupDeviceZ(SvPositionToBufferUV(SvPosition + float4(0, 0, 0, 0)));
	float DeviceZLeft = LookupDeviceZ(SvPositionToBufferUV(SvPosition + float4(-1, 0, 0, 0)));
	float DeviceZTop = LookupDeviceZ(SvPositionToBufferUV(SvPosition + float4(0, -1, 0, 0)));
	float DeviceZRight = LookupDeviceZ(SvPositionToBufferUV(SvPosition + float4(1, 0, 0, 0)));
	float DeviceZBottom = LookupDeviceZ(SvPositionToBufferUV(SvPosition + float4(0, 1, 0, 0)));

	// Favor the surfae we are looking at. Simiar to: http://www.humus.name/index.php?page=3D&ID=84
	float DeviceZDdx = TakeSmallerAbsDelta(DeviceZLeft, DeviceZ, DeviceZRight);
	float DeviceZDdy = TakeSmallerAbsDelta(DeviceZTop, DeviceZ, DeviceZBottom);

	// can be optimized, is not fully centered but that should not matter much
	float3 Mid =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(0, 0), DeviceZ, 1));
	float3 Right =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(1, 0), DeviceZ + DeviceZDdx, 1)) - Mid;
	float3 Down =	SvPositionToTranslatedWorld(float4(SvPosition.xy + float2(0, 1), DeviceZ + DeviceZDdy, 1)) - Mid;

	return cross(Right, Down);
}

// @return can be 0,0,0 if we don't have a good input normal
float3 GetWorldSpaceNormalFromAOInput(float2 UV, float4 SvPosition)
{
	float3 WorldNormal = 0;

	if (USE_NORMALS)
	{
	#if USE_AO_SETUP_AS_INPUT
		// Low resolution normal computed in the setup (downscaled) pass.
		WorldNormal = Texture2DSample(SSAO_SetupTexture, SSAO_Sampler, frac(UV)).xyz * 2 - 1;
	#elif COMPUTE_SHADER || FORWARD_SHADING || (SHADING_PATH_MOBILE && (!MOBILE_DEFERRED_SHADING || ALLOW_FRAMEBUFFER_FETCH))
		// Async compute and forward shading don't have access to the gbuffer.
		// for mobile case only multi-pass defferred can access gbuffer
		WorldNormal = ReconstructNormalFromDepthBuffer(SvPosition);
	#else
#if SUBSTRATE_GBUFFER_FORMAT==1
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(SvPosition.xy, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstrateHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		if (SubstrateHeader.SubstrateGetBSDFType() != SUBSTRATE_BSDF_TYPE_HAIR)
		{
			const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SvPosition.xy, 0)));
			WorldNormal = TopLayerData.WorldNormal;
		}
#else
		// Otherwise sample Gbuffer normals if the shader model has normals.
		FGBufferData GBuffer = GetGBufferData(UV, false);
		bool bUseNormal = (GBuffer.ShadingModelID != SHADINGMODELID_HAIR);
		#if	USE_SM_EXCLUDE_MASK
			bUseNormal = bUseNormal & (((1u << GBuffer.ShadingModelID) & SSAO_ShadingModelsExcludeMask) == 0u);
		#endif
		if (bUseNormal)
		{
			WorldNormal = GBuffer.WorldNormal;
		}
#endif
	#endif
	}

	return WorldNormal;
}

float4 ComputeUpsampleContribution(float SceneDepth, float2 InUV, float3 CenterWorldNormal)
{
	// can be optimized
#if AO_UPSAMPLE_QUALITY == 0
	const int SampleCount = 4;
	float2 UV[SampleCount];

	UV[0] = InUV + float2(-0.5f,  0.5f) * SSAO_DownsampledAOInverseSize;
	UV[1] = InUV + float2( 0.5f,  0.5f) * SSAO_DownsampledAOInverseSize;
	UV[2] = InUV + float2(-0.5f, -0.5f) * SSAO_DownsampledAOInverseSize;
	UV[3] = InUV + float2( 0.5f, -0.5f) * SSAO_DownsampledAOInverseSize;
#else // AO_UPSAMPLE_QUALITY == 0
	const int SampleCount = 9;
	float2 UV[SampleCount];

	UV[0] = InUV + float2( -1, -1) * SSAO_DownsampledAOInverseSize;
	UV[1] = InUV + float2(  0, -1) * SSAO_DownsampledAOInverseSize;
	UV[2] = InUV + float2(  1, -1) * SSAO_DownsampledAOInverseSize;
	UV[3] = InUV + float2( -1,  0) * SSAO_DownsampledAOInverseSize;
	UV[4] = InUV + float2(  0,  0) * SSAO_DownsampledAOInverseSize;
	UV[5] = InUV + float2(  1,  0) * SSAO_DownsampledAOInverseSize;
	UV[6] = InUV + float2( -1,  1) * SSAO_DownsampledAOInverseSize;
	UV[7] = InUV + float2(  0,  1) * SSAO_DownsampledAOInverseSize;
	UV[8] = InUV + float2(  1,  1) * SSAO_DownsampledAOInverseSize;
#endif // AO_UPSAMPLE_QUALITY == 0

	// to avoid division by 0
	float SmallValue = 0.0001f;

	// we could weight the samples better but tests didn't show much difference
	float WeightSum = SmallValue;
	float4 Ret = float4(SmallValue,0,0,0);

	float InvThreshold = ScreenSpaceAOParams[2].y;
	float MinIteration = 1.0f;

	UNROLL for(int i = 0; i < SampleCount; ++i)
	{
#if SHADING_PATH_MOBILE
		float2 ClampedUV = UV[i];
#else
		float2 ClampedUV = clamp(UV[i], SSAO_DownsampledAOUVViewportMin, SSAO_DownsampledAOUVViewportMax);
#endif
		float4 SampleValue = Texture2DSample(SSAO_DownsampledAO, SSAO_Sampler, ClampedUV);

		MinIteration = min(MinIteration, SampleValue.g);

		float4 NormalAndSampleDepth = Texture2DSample(SSAO_NormalsTexture, SSAO_Sampler, ClampedUV);
		float SampleDepth = NormalAndSampleDepth.a * Constant_Float16F_Scale;

		// when tweaking this constant look for crawling pattern at edges
		float Weight = ComputeDepthSimilarity(SampleDepth, SceneDepth, 0.003f);

		if (USE_NORMALS)
		{
			float3 LocalWorldNormal = NormalAndSampleDepth.xyz*2-1;
			Weight *= saturate(dot(LocalWorldNormal, CenterWorldNormal));
		}

		// todo: 1 can be put into the input to save an instruction
		Ret += float4(SampleValue.rgb, 1) * Weight;
		WeightSum += Weight;
	}

	Ret /= WeightSum;
	Ret.g = MinIteration;

	return Ret;
}

// to blend between upsampled and current pass data
float ComputeLerpFactor()
{
	// set up on C++ side
	float MipBlend = ScreenSpaceAOParams[2].w;

	float AOLerpFactor = MipBlend;

#if AO_SAMPLE_QUALITY == 0
	// we have no AO, we only use the upsampled data
	AOLerpFactor = 1.0f;
#endif

#if USE_UPSAMPLE == 0
	// if there is no former pass we cannot use the data
	AOLerpFactor = 0.0f;
#endif
	
	return AOLerpFactor;
}

// @return NormAngle means 0..1 is actually 0..PI
float acosApproxNormAngle(float x)
{
	// todo: expose
	// 1: is a good linear approximation, 0.9f seems to look good
	float ContrastTweak = 0.9f;

	// correct: acos(x) / PI
	// linear approximation: saturate((1 - x) * 0.5f);
	// pretty good approximation with contrast tweak
	return saturate((1 - x) * 0.5f * ContrastTweak);
}

// @return float3(InvNormAngleL, InvNormAngleR, Weight)
float3 WedgeWithNormal(float2 ScreenSpacePosCenter, float2 InLocalRandom, float3 InvFovFix, float3 ViewSpacePosition, float3 ScaledViewSpaceNormal, float InvHaloSize, float MipLevel)
{
	float2 ScreenSpacePosL = ScreenSpacePosCenter + InLocalRandom;
	float2 ScreenSpacePosR = ScreenSpacePosCenter - InLocalRandom;

	float AbsL = GetHZBDepth(ScreenSpacePosL, MipLevel);
	float AbsR = GetHZBDepth(ScreenSpacePosR, MipLevel);

	float3 SamplePositionL = ReconstructCSPos(AbsL, ScreenSpacePosL);
	float3 SamplePositionR = ReconstructCSPos(AbsR, ScreenSpacePosR);

	float3 DeltaL = (SamplePositionL - ViewSpacePosition) * InvFovFix;
	float3 DeltaR = (SamplePositionR - ViewSpacePosition) * InvFovFix;
		
#if OPTIMIZATION_O1
	float InvNormAngleL = saturate(dot(DeltaL, ScaledViewSpaceNormal) / dot(DeltaL, DeltaL));
	float InvNormAngleR = saturate(dot(DeltaR, ScaledViewSpaceNormal) / dot(DeltaR, DeltaR));
	float Weight = 1;
#else
	float InvNormAngleL = saturate(dot(DeltaL, ScaledViewSpaceNormal) * rsqrt(dot(DeltaL, DeltaL)));
	float InvNormAngleR = saturate(dot(DeltaR, ScaledViewSpaceNormal) * rsqrt(dot(DeltaR, DeltaR)));

	float Weight = 
		  saturate(1.0f - length(DeltaL) * InvHaloSize)
		* saturate(1.0f - length(DeltaR) * InvHaloSize);
#endif

	return float3(InvNormAngleL, InvNormAngleR, Weight);
}



// @return float2(InvNormAngle, Weight)
float2 WedgeNoNormal(float2 ScreenSpacePosCenter, float2 InLocalRandom, float3 InvFovFix, float3 ViewSpacePosition, float InvHaloSize, float MipLevel)
{
	float2 ScreenSpacePosL = ScreenSpacePosCenter + InLocalRandom;
	float2 ScreenSpacePosR = ScreenSpacePosCenter - InLocalRandom;

	float AbsL = GetHZBDepth(ScreenSpacePosL, MipLevel);
	float AbsR = GetHZBDepth(ScreenSpacePosR, MipLevel);
	
	float3 SamplePositionL = ReconstructCSPos(AbsL, ScreenSpacePosL);
	float3 SamplePositionR = ReconstructCSPos(AbsR, ScreenSpacePosR);
	
	float3 DeltaL = (SamplePositionL - ViewSpacePosition) * InvFovFix;
	float3 DeltaR = (SamplePositionR - ViewSpacePosition) * InvFovFix;

	float WeightLeft;
	float3 SamplePositionLeft;
	{
		WeightLeft = 1;

#if !OPTIMIZATION_O1
		WeightLeft = saturate(1.0f - length(DeltaL) * InvHaloSize);
#endif
	}

	float WeightRight;
	float3 SamplePositionRight;
	{	
		WeightRight = 1;

#if !OPTIMIZATION_O1
		WeightRight = saturate(1.0f - length(DeltaR) * InvHaloSize);
#endif
	}


	float FlatSurfaceBias = 5.0f;

	float left = ViewSpacePosition.z - AbsL;
	float right = ViewSpacePosition.z - AbsR;

	// OptionA: accurate angle computation
	float NormAngle = acosApproxNormAngle( dot(DeltaL, DeltaR) / sqrt(length2(DeltaL) * length2(DeltaR)));
	// OptionB(fade out in near distance): float NormAngle = acosApproxNormAngle( (- left - right) * 20);
	// OptionC(look consistent but more noisy, should be much faster): float NormAngle = 0;


	// not 100% correct but simple
	// bias is needed to avoid flickering on almost perfectly flat surfaces
	//	    if((leftAbs  + rightAbs) * 0.5f > SceneDepth - 0.0001f)
	if(left + right < FlatSurfaceBias)
	{
		// fix concave case
		NormAngle = 1;
	}

	// to avoid halos around objects
	float Weight = 1;
				
	float InvAmbientOcclusionDistance = ScreenSpaceAOParams[0].z;
	float ViewDepthAdd = 1.0f - ViewSpacePosition.z * InvAmbientOcclusionDistance;

	Weight *= saturate(SamplePositionL.z * InvAmbientOcclusionDistance + ViewDepthAdd);
	Weight *= saturate(SamplePositionR.z * InvAmbientOcclusionDistance + ViewDepthAdd);

//	return float2(1 - NormAngle, (WeightLeft + WeightRight) * 0.5f);
	return float2((1-NormAngle) / (Weight + 0.001f), Weight);
}

float3 ReconstructNormal(float2 In)
{
	return float3(In, sqrt(1 - dot(In, In)));
}


// @param ScreenSpacePos -1..1
// @return 1 if inside the center, 0 if outside
float ComputeSampleDebugMask(float2 ScreenSpacePos, float MipLevel)
{
	ScreenSpacePos.x -= 0.5f;

	ScreenSpacePos.y = frac(ScreenSpacePos.y) - 0.5f;

	float2 ViewPortSize = AOViewport_ViewportSize;
	int2 PixelOffsetToCenter = int2(ScreenSpacePos * ViewPortSize * 0.5f);

	float d = length(PixelOffsetToCenter);

	// revisit this
	float radius = 12.0f;

	// hard
	return d < radius * exp2(MipLevel);
	// soft
//	return saturate(1 - d / (radius * exp2(MipLevel)));
}

float ComputeMipLevel(int sampleid, int step)
{
	float SamplePos = (sampleid + 0.5f) / SAMPLESET_ARRAY_SIZE;

	float HzbStepMipLevelFactorValue = ScreenSpaceAOParams[4].z;
	// use a constant to get better performance
	//float HzbStepMipLevelFactorValue = 0.5f;
//	float HzbStepMipLevelFactorValue = 1;

	float Scale = (step + 1) / (float)SAMPLE_STEPS;

//	return log2(1.0f + HzbStepMipLevelFactorValue * Scale * SamplePos);
	return log2(HzbStepMipLevelFactorValue * Scale * SamplePos);
}

float GetDepthFromAOInput(float2 UV)
{	
#if USE_AO_SETUP_AS_INPUT
	// low resolution
	return Texture2DSample(SSAO_SetupTexture, SSAO_Sampler, UV).a * Constant_Float16F_Scale;
#else
	// full resolution 
	return CalcSceneDepth(UV);
#endif
}

float2 UVToScreenPos(float2 UV, float2 Extent)
{
	return (UV * Extent - AOViewport_ScreenPosToViewportBias) / AOViewport_ScreenPosToViewportScale;
}
