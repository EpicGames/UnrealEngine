// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessBloom.usf: PostProcessing bloom
=============================================================================*/

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "EyeAdaptationCommon.ush"
#include "PostProcessHistogramCommon.ush"
#include "ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

Texture2D InputTexture;
SamplerState InputSampler;

Texture3D LumBilateralGrid;
SamplerState LumBilateralGridSampler;
Texture2D BlurredLogLum;
SamplerState BlurredLogLumSampler;

float BloomThreshold;

float2 GetExposureScaleMiddleGreyLumValue()
{
	float2 ExposureScaleMiddleGreyLumValue = EyeAdaptationLookupBuffer(EyeAdaptationBuffer).xw;

	// Middle grey lum value adjusted by exposure compensation
	ExposureScaleMiddleGreyLumValue.y = log2(0.18f * ExposureScaleMiddleGreyLumValue.y * LocalExposure_MiddleGreyExposureCompensation);

	return ExposureScaleMiddleGreyLumValue;
}

float4 BloomSetupCommon(float2 UV, float2 ViewportUV, float2 ExposureScaleMiddleGreyLumValue)
{
	float4 SceneColor = Texture2DSample(InputTexture, InputSampler, UV) * View.OneOverPreExposure;

	float ExposureScale = ExposureScaleMiddleGreyLumValue.x;

#if USE_LOCAL_EXPOSURE
	{
		float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor.rgb);
		float LogLuminance = log2(LuminanceVal);
		float MiddleGreyLumValue = ExposureScaleMiddleGreyLumValue.y;

		float BaseLogLum = CalculateBaseLogLuminance(LogLuminance, LocalExposure_BlurredLuminanceBlend, ExposureScale, ViewportUV, LumBilateralGrid, BlurredLogLum, LumBilateralGridSampler, BlurredLogLumSampler);
		float LocalExposure = CalculateLocalExposure(LogLuminance + log2(ExposureScale), BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, LocalExposure_DetailStrength);
		
		SceneColor.rgb *= LocalExposure;
	}
#endif

	half3 LinearColor = SceneColor.rgb;

#if USE_THRESHOLD
	half TotalLuminance = Luminance(LinearColor) * ExposureScale;
	half BloomLuminance = TotalLuminance - BloomThreshold;
	half BloomAmount = saturate(BloomLuminance * 0.5f);
#else
	half BloomAmount = 1.0f;
#endif

	return float4(BloomAmount * LinearColor, 0) * View.PreExposure;
}

#if PIXELSHADER

FScreenTransform SvPositionToInputTextureUV;

void BloomSetupPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 UV = ApplyScreenTransform(SvPosition.xy, SvPositionToInputTextureUV);
	float2 ViewportUV = (int2(SvPosition.xy) - Input_ViewportMin) * Input_ViewportSizeInverse;
	
	float2 ExposureScaleMiddleGreyLumValue = GetExposureScaleMiddleGreyLumValue();

	OutColor = BloomSetupCommon(UV, ViewportUV, ExposureScaleMiddleGreyLumValue);
}

#elif COMPUTESHADER

RWTexture2D<float4> RWOutputTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void BloomSetupCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float2 UV = ((float2)DispatchThreadId + (float2)Input_ViewportMin + 0.5f) * Input_ExtentInverse;

	if (IsComputeUVOutOfBounds(UV))
	{
		return;
	}

	float2 ViewportUV = DispatchThreadId * Input_ViewportSizeInverse;

	float2 ExposureScaleMiddleGreyLumValue = GetExposureScaleMiddleGreyLumValue();

	float4 OutColor = BloomSetupCommon(UV, ViewportUV, ExposureScaleMiddleGreyLumValue);

	uint2 PixelPos = DispatchThreadId + Input_ViewportMin;

	RWOutputTexture[PixelPos] = OutColor;
}

#endif