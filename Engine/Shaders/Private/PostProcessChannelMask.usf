// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessChannelMask.usf: Masks to a single channel and outputs it as greyscale.
=============================================================================*/

#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "ScreenPass.ush"

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

float4 ChannelMask;
uint bMaskChannels;
uint bInvertAlphaChannelMask;
uint bDisplayCheckerboard;
FScreenTransform ScreenPosToViewPos;

#define CheckerSize 16
#define InvCheckerSize (1.0f / CheckerSize)
#define CheckerWhite float3(0.216f, 0.216f, 0.216f)
#define CheckerGray float3(0.033f, 0.033f, 0.033f)

float3 GetCheckerboardColor(float2 PixelPos)
{
	float2 ViewPos = ApplyScreenTransform(PixelPos, ScreenPosToViewPos);
	int2 CheckerCell = floor(ViewPos * float2(InvCheckerSize, InvCheckerSize) * View.ResolutionFractionAndInv.y);
	return CheckerCell.x % 2 == CheckerCell.y % 2 ? CheckerWhite : CheckerGray;
}

void ChannelMask_MainPS(in noperspective float4 UVAndScreenPos : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	float4 SceneColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UVAndScreenPos.xy);

	// Always output the original scene alpha, regardless of what the output rgb ends up being
	OutColor.a = SceneColor.a;
	
	// Non-alpha channels will be encoded in sRGB by this point, so to properly composite with checkerboard, convert to linear space
	// TODO: Native HDR (linear ScRGB or Rec2020 PQ) or gamma 2.2 will need to be subsequently supported
	SceneColor.rgb = sRGBToLinear(SceneColor.rgb);
	
	// Invert the alpha if needed
	SceneColor.a = bInvertAlphaChannelMask ? saturate(1.0 - SceneColor.a) : SceneColor.a;
	
	float3 Channel = bMaskChannels ? dot(SceneColor.rgba, ChannelMask) : SceneColor.rgb;
	float3 CheckerboardColor = bDisplayCheckerboard ? GetCheckerboardColor(UVAndScreenPos.zw) : 0.0f;

	// Perform a naive premult alpha composite with the raw contents of the scene color's alpha.
	// When not inverting the alpha, this is a technically incorrect blending, as Channel has been premultiplied with 1.0 - SceneColor.a
	// and not the correct SceneColor.a (to show the correct value, the scene color would need to be un-pre-multiplied and re-pre-multiplied by the correct alphas)
	// but we will simply output the "wrong" values to demonstrate the incorrectness
	OutColor.rgb = LinearToSrgb(Channel + CheckerboardColor * (1.0 - SceneColor.a));
}