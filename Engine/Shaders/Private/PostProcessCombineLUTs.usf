// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessCombineLUTs.usf: Filter pixel shader source.

	Note: Any changes to device encoding logic in CombineLUTsCommon() must
	also be made in PostProcessDeviceEncodingOnly.usf.
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "TonemapCommon.ush"

// ---------------------------------------------------

// Texture0 is the neutral one and is computed in the shader
Texture2D Textures_1;
SamplerState Samplers_1;
Texture2D Textures_2;
SamplerState Samplers_2;
Texture2D Textures_3;
SamplerState Samplers_3;
Texture2D Textures_4;
SamplerState Samplers_4;
// 0 is for neutral, 1 for Textures_1, 2 for ...
DECLARE_SCALAR_ARRAY(float, LUTWeights, 5);
half3 ColorScale;
half4 OverlayColor;

float3 LimitingRgbToXYZ0;
float3 LimitingRgbToXYZ1;
float3 LimitingRgbToXYZ2;
float3x3 GetLimitingRgbToXYZ()
{
	return float3x3(LimitingRgbToXYZ0, LimitingRgbToXYZ1, LimitingRgbToXYZ2);
}

float3 LimitingXYZToRgb0;
float3 LimitingXYZToRgb1;
float3 LimitingXYZToRgb2;
float3x3 GetLimitingXYZToRgb()
{
	return float3x3(LimitingXYZToRgb0, LimitingXYZToRgb1, LimitingXYZToRgb2);
}

uint bIsTemperatureWhiteBalance;

float LUTInvMax;
float LUTSize;
float InnerLUTSize;

float WhiteTemp;
float WhiteTint;

float4 ColorSaturation;
float4 ColorContrast;
float4 ColorGamma;
float4 ColorGain;
float4 ColorOffset;

float4 ColorSaturationShadows;
float4 ColorContrastShadows;
float4 ColorGammaShadows;
float4 ColorGainShadows;
float4 ColorOffsetShadows;

float4 ColorSaturationMidtones;
float4 ColorContrastMidtones;
float4 ColorGammaMidtones;
float4 ColorGainMidtones;
float4 ColorOffsetMidtones;

float4 ColorSaturationHighlights;
float4 ColorContrastHighlights;
float4 ColorGammaHighlights;
float4 ColorGainHighlights;
float4 ColorOffsetHighlights;

float  ColorCorrectionShadowsMax;
float  ColorCorrectionHighlightsMin;
float  ColorCorrectionHighlightsMax;

static const float DefaultTemperature = 6500.0;
static const float DefaultTint = 0.0;
static const float Epsilon = 1e-8;
static const float Bt1886Gamma = 2.4;

float3 ColorCorrect( float3 WorkingColor,
	float4 ColorSaturation,
	float4 ColorContrast,
	float4 ColorGamma,
	float4 ColorGain,
	float4 ColorOffset )
{
	// TODO optimize
	float Luma = dot( WorkingColor, AP1_RGB2Y );
	WorkingColor = max( 0, lerp( Luma.xxx, WorkingColor, ColorSaturation.xyz*ColorSaturation.w ) );
	WorkingColor = pow( WorkingColor * (1.0 / 0.18), ColorContrast.xyz*ColorContrast.w ) * 0.18;
	WorkingColor = pow( WorkingColor, 1.0 / (ColorGamma.xyz*ColorGamma.w) );
	WorkingColor = WorkingColor * (ColorGain.xyz * ColorGain.w) + (ColorOffset.xyz + ColorOffset.w);
	return WorkingColor;
}

// Nuke-style Color Correct
float3 ColorCorrectAll( float3 WorkingColor )
{
	float Luma = dot( WorkingColor, AP1_RGB2Y );

	// Shadow CC
	float3 CCColorShadows = ColorCorrect(WorkingColor, 
		ColorSaturationShadows*ColorSaturation, 
		ColorContrastShadows*ColorContrast, 
		ColorGammaShadows*ColorGamma, 
		ColorGainShadows*ColorGain, 
		ColorOffsetShadows+ColorOffset);
	float CCWeightShadows = 1- smoothstep(0, ColorCorrectionShadowsMax, Luma);
	
	// Highlight CC
	float3 CCColorHighlights = ColorCorrect(WorkingColor, 
		ColorSaturationHighlights*ColorSaturation, 
		ColorContrastHighlights*ColorContrast, 
		ColorGammaHighlights*ColorGamma, 
		ColorGainHighlights*ColorGain, 
		ColorOffsetHighlights+ColorOffset);
	float CCWeightHighlights = smoothstep(ColorCorrectionHighlightsMin, ColorCorrectionHighlightsMax, Luma);

	// Midtone CC
	float3 CCColorMidtones = ColorCorrect(WorkingColor, 
		ColorSaturationMidtones*ColorSaturation, 
		ColorContrastMidtones*ColorContrast, 
		ColorGammaMidtones*ColorGamma, 
		ColorGainMidtones*ColorGain, 
		ColorOffsetMidtones+ColorOffset);
	float CCWeightMidtones = 1 - CCWeightShadows - CCWeightHighlights;

	// Blend Shadow, Midtone and Highlight CCs
	float3 WorkingColorSMH = CCColorShadows*CCWeightShadows + CCColorMidtones*CCWeightMidtones + CCColorHighlights*CCWeightHighlights;
	
	return WorkingColorSMH;
}

uint OutputDevice;
uint OutputGamut;
float OutputMaxLuminance;
uint LUTShaper;

uint GetOutputDevice()
{
#if OUTPUT_DEVICE_SRGB
	return TONEMAPPER_OUTPUT_sRGB;
#else
	return OutputDevice;
#endif
}

float BlueCorrection;
float ExpandGamut;
float ToneCurveAmount;

float3 ComputeFilmColorNoGamma(float3 ColorAP1)
{
	const float3x3 AP0_2_AP1 = mul( XYZ_2_AP1_MAT, AP0_2_XYZ_MAT );
	const float3x3 AP1_2_AP0 = mul( XYZ_2_AP0_MAT, AP1_2_XYZ_MAT );

	const float3x3 BlueCorrect =
	{
		0.9404372683, -0.0183068787, 0.0778696104,
		0.0083786969,  0.8286599939, 0.1629613092,
		0.0005471261, -0.0008833746, 1.0003362486
	};
	const float3x3 BlueCorrectInv =
	{
		 1.06318,     0.0233956, -0.0865726,
		-0.0106337,   1.20632,   -0.19569,
		-0.000590887, 0.00105248, 0.999538
	};
	const float3x3 BlueCorrectAP1    = mul( AP0_2_AP1, mul( BlueCorrect,    AP1_2_AP0 ) );
	const float3x3 BlueCorrectInvAP1 = mul( AP0_2_AP1, mul( BlueCorrectInv, AP1_2_AP0 ) );

	// Blue correction
	ColorAP1 = lerp( ColorAP1, mul( BlueCorrectAP1, ColorAP1 ), BlueCorrection );

	// Tonemapped color in the AP1 gamut
	float3 ToneMappedColorAP1 = FilmToneMap( ColorAP1 );
	ColorAP1 = lerp(ColorAP1, ToneMappedColorAP1, ToneCurveAmount);

	// Uncorrect blue to maintain white point
	ColorAP1 = lerp( ColorAP1, mul( BlueCorrectInvAP1, ColorAP1 ), BlueCorrection );

	// Convert from AP1 to the working color space and clip out-of-gamut values
	float3 FilmColor = max(0, mul( (float3x3)WorkingColorSpace.FromAP1, ColorAP1 ));

#if BLENDCOUNT > 1
	{
		// Legacy LDR LUT color grading

		// FIXME 
		// LUTs are in sRGB transfer function space
		// LUTs applied in sRGB gamut as well
		half3 GammaColor = LinearToSrgb( saturate( FilmColor ) );

		float3 UVW = GammaColor * (15.0 / 16.0) + (0.5f / 16.0);

		GammaColor = GET_SCALAR_ARRAY_ELEMENT(LUTWeights, 0) * GammaColor;

		// BLENDCOUNT is the number of LUT that are blended together including the neutral one
	#if BLENDCOUNT >= 2
		GammaColor += GET_SCALAR_ARRAY_ELEMENT(LUTWeights, 1) * UnwrappedTexture3DSample( Textures_1, Samplers_1, UVW, 16, 1 / 16.0 ).rgb;
	#endif

	#if BLENDCOUNT >= 3
		GammaColor += GET_SCALAR_ARRAY_ELEMENT(LUTWeights, 2) * UnwrappedTexture3DSample( Textures_2, Samplers_2, UVW, 16, 1 / 16.0 ).rgb;
	#endif

	#if BLENDCOUNT >= 4
		GammaColor += GET_SCALAR_ARRAY_ELEMENT(LUTWeights, 3) * UnwrappedTexture3DSample( Textures_3, Samplers_3, UVW, 16, 1 / 16.0 ).rgb;
	#endif

	#if BLENDCOUNT >= 5
		GammaColor += GET_SCALAR_ARRAY_ELEMENT(LUTWeights, 4) * UnwrappedTexture3DSample( Textures_4, Samplers_4, UVW, 16, 1 / 16.0 ).rgb;
	#endif

		// Back to linear space, AP1 gamut
		FilmColor =  sRGBToLinear( GammaColor );
	}
#endif
	
	// apply math color correction on top to texture based solution
	FilmColor = ColorCorrection( FilmColor );

	// blend with custom LDR color, used for Fade track in Cinematics
	float3 FilmColorNoGamma = lerp( FilmColor * ColorScale, OverlayColor.rgb, OverlayColor.a );
	return FilmColorNoGamma;
}

float3 ComputeGradedColor(float3 ColorAP1)
{
	// Store for Linear HDR output without tone curve
	float3 GradedColor = mul( (float3x3)WorkingColorSpace.FromAP1, ColorAP1 );

	// blend with custom LDR color, used for Fade track in Cinematics
	GradedColor = lerp(GradedColor * ColorScale, OverlayColor.rgb, OverlayColor.a);

	return GradedColor;
}

#define INVEOTF_LINEAR 0
#define INVEOTF_PQ 1

float3 ApplyInvEOTF(float3 Color, int InvEOTF)
{
	if (InvEOTF == INVEOTF_PQ)
	{
		// Apply conversion to ST-2084 (Dolby PQ)
		Color = LinearToST2084(Color * LinearToNitsScale);
	}

	return Color;
}

Texture3D InnerLUT;
SamplerState InnerLUTSampler;

float3 InnerLookupTable(float3 LUTEncodedColor, int InvEOTF)
{
	float InvInnerLUTSize = 1. / InnerLUTSize;
	float InnerLUTScale = 1. - InvInnerLUTSize;
	float InnerLUTOffset = .5 * InvInnerLUTSize;
	float3 UVW = mad(LUTEncodedColor, InnerLUTScale, InnerLUTOffset);

	float3 Color = Texture3DSample(InnerLUT, InnerLUTSampler, UVW).rgb;

	if (OutputMaxLuminance > LinearToNitsScale)
	{
		if (InvEOTF != INVEOTF_PQ)
		{
			Color = ST2084ToLinear(Color) * LinearToNitsScaleInverse;
			Color = ApplyInvEOTF(Color, InvEOTF);
		}
	}
	else
	{
		Color = sRGBToLinear(Color);
		Color = ApplyInvEOTF(Color, InvEOTF);
	}

	return Color;
}

#define TONEMAPPING_SDR_INVEOTF_SRGB 0
#define TONEMAPPING_SDR_INVEOTF_BT1886 1
uint TonemappingSDRInvEOTF;

float3 ApplyToneCurve(float3 ColorAP1, float PeakLuminance, float3x3 AP1_2_Output, int InvEOTF)
{
#if TONEMAPPING_METHOD == TONEMAPPING_METHOD_ACES
	float3 GradedColor = ComputeGradedColor(ColorAP1);
	float3 GradedColorAP1 = mul( (float3x3)WorkingColorSpace.ToAP1, GradedColor );
	float3 LUTEncodedColor = LinearToST2084(clamp(GradedColorAP1 * LinearToNitsScale, 0.0, 10000.0));
	float3 OutputGamutColor = InnerLookupTable(LUTEncodedColor, InvEOTF);
#elif TONEMAPPING_METHOD == TONEMAPPING_METHOD_Filmic
	float3 FilmColorNoGamma = ComputeFilmColorNoGamma(ColorAP1);

	// Apply "gamma" curve adjustment.
	float3 FilmColor = pow( max(0, FilmColorNoGamma), InverseGamma.y );

	// FIXME: Workaround for UE-29935, pushing all colors with a 0 component to black output
	// Default parameters seem to cancel out (sRGB->XYZ->AP1->XYZ->sRGB), so should be okay for a temp fix
	float3 OutputGamutColor = WorkingColorSpace.bIsSRGB && (OutputGamut == TONEMAPPER_GAMUT_sRGB_D65) ?
		FilmColor :
		mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, FilmColor ) );

#if !OUTPUT_DEVICE_SRGB
	// "Automatic HDR" using ACES to remap filmic.
	if (PeakLuminance > LinearToNitsScale)
	{
		// Rec. 2100 specifies a dim environment similar to various viewing standards, e.g. BT.2035, for consuming Rec. 709 w/BT.1886.
		// sRGB is meant to be viewed in a bright office, so its EOTF leaves values a bit inflated in comparison to BT.1886.
		// This adjustment converts from "bright" linear space to "dim" linear space.
		if (TonemappingSDRInvEOTF == TONEMAPPING_SDR_INVEOTF_SRGB)
		{
			// Apply inverse of sRGB EOTF to generate nonlinear values that would get sent to an SDR display.
			OutputGamutColor = LinearToSrgb(OutputGamutColor);
			// Apply BT.1886 EOTF to linearize the values into "dim" linear space.
			OutputGamutColor = pow(OutputGamutColor, Bt1886Gamma);
		}

		// Use PQ shaper to preserve shadow detail and normalize for SDR by using the reciprocal of ST2084 at 100 nits.
		float3 LUTEncodedColor = LinearToST2084(clamp(OutputGamutColor * LinearToNitsScale, 0.0, 10000.0)) * 1.9682001;
		OutputGamutColor = InnerLookupTable(LUTEncodedColor, InvEOTF);
	}
	else
#endif
	{
		OutputGamutColor = ApplyInvEOTF(OutputGamutColor, InvEOTF);
	}
#else
#error Unknown TONEMAPPING_METHOD
#endif

	return OutputGamutColor;
}

float4 CombineLUTsCommon(float3 LUTEncodedColor)
{
	float4 OutColor = 0;

	const float3x3 AP1_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	const float3x3 AP1_2_Output = OutputGamutMappingMatrix( OutputGamut );

	float3 LinearColor;
	// Decode texture values as ST-2084 (Dolby PQ)
	BRANCH
	if (LUTShaper)
	{
		// Since ST2084 returns linear values in nits, divide by a scale factor to convert
		// the reference nit result to be 1.0 in linear.
		// (for efficiency multiply by precomputed inverse)
		LinearColor = ST2084ToLinear(LUTEncodedColor) * LinearToNitsScaleInverse;
	}
	// Decode log values
	else
	{
		LinearColor = LogToLin(LUTEncodedColor) - LogToLin(0);
	}

	float3 BalancedColor = LinearColor;

	BRANCH
	if( abs(WhiteTemp - DefaultTemperature) > Epsilon || abs(WhiteTint - DefaultTint) > Epsilon )
	{
		BalancedColor = WhiteBalance(LinearColor, WhiteTemp, WhiteTint, bIsTemperatureWhiteBalance, (float3x3)WorkingColorSpace.ToXYZ, (float3x3)WorkingColorSpace.FromXYZ);
	}

	float3 ColorAP1 = mul( (float3x3)WorkingColorSpace.ToAP1, BalancedColor );

	// Expand bright saturated colors outside the sRGB gamut to fake wide gamut rendering.
 	float  LumaAP1 = dot( ColorAP1, AP1_RGB2Y );
	float3 ChromaAP1 = ColorAP1 / LumaAP1;

	float ChromaDistSqr = dot( ChromaAP1 - 1, ChromaAP1 - 1 );
	float ExpandAmount = ( 1 - exp2( -4 * ChromaDistSqr ) ) * ( 1 - exp2( -4 * ExpandGamut * LumaAP1*LumaAP1 ) );

	// Bizarre matrix but this expands sRGB to between P3 and AP1
	// CIE 1931 chromaticities:	x		y
	//				Red:		0.6965	0.3065
	//				Green:		0.245	0.718
	//				Blue:		0.1302	0.0456
	//				White:		0.3127	0.329
	const float3x3 Wide_2_XYZ_MAT = 
	{
		0.5441691,  0.2395926,  0.1666943,
		0.2394656,  0.7021530,  0.0583814,
		-0.0023439,  0.0361834,  1.0552183,
	};

	const float3x3 Wide_2_AP1 = mul( XYZ_2_AP1_MAT, Wide_2_XYZ_MAT );
	const float3x3 ExpandMat = mul( Wide_2_AP1, AP1_2_sRGB );

	float3 ColorExpand = mul( ExpandMat, ColorAP1 );
	ColorAP1 = lerp( ColorAP1, ColorExpand, ExpandAmount );

	ColorAP1 = ColorCorrectAll( ColorAP1 );

	// Note: Any changes to the following device encoding logic must also be made 
	// in PostProcessDeviceEncodingOnly.usf's corresponding pixel shader.
	half3 OutDeviceColor = 0;

	// sRGB, user specified gamut
	BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_sRGB)
	{
		float3 OutputGamutColor = ApplyToneCurve(ColorAP1, LinearToNitsScale, AP1_2_Output, INVEOTF_LINEAR);

		BRANCH if (TonemappingSDRInvEOTF == TONEMAPPING_SDR_INVEOTF_SRGB)
		{
			OutDeviceColor = LinearToSrgb(OutputGamutColor);
		}
		else // TonemappingSDRInvEOTF == TONEMAPPING_SDR_INVEOTF_BT1886
		{
			OutDeviceColor = pow(OutputGamutColor, 1. / Bt1886Gamma);
		}
	}

	// Rec 709, user specified gamut
	else BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_Rec709)
	{
		float3 OutputGamutColor = ApplyToneCurve(ColorAP1, LinearToNitsScale, AP1_2_Output, INVEOTF_LINEAR);

		// Didn't profile yet if the branching version would be faster (different linear segment).
		OutDeviceColor = LinearTo709Branchless(OutputGamutColor);
	}

	// Transform with PQ/2084 encoding, user specified gamut
	else BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_ACES1000nitST2084 || GetOutputDevice() == TONEMAPPER_OUTPUT_ACES2000nitST2084)
	{
		OutDeviceColor = ApplyToneCurve(ColorAP1, OutputMaxLuminance, AP1_2_Output, INVEOTF_PQ);
	}

	// Transform to linear ScRGB
	else BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_ACES1000nitScRGB || GetOutputDevice() == TONEMAPPER_OUTPUT_ACES2000nitScRGB)
	{
		float3 OutputGamutColor = ApplyToneCurve(ColorAP1, OutputMaxLuminance, AP1_2_Output, INVEOTF_LINEAR);
		OutDeviceColor = (LinearToNitsScale / UE_SCRGB_WHITE_NITS) * OutputGamutColor;
	}

	else BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_LinearEXR)
	{
		float3 GradedColor = ComputeGradedColor(ColorAP1);
		float3 OutputGamutColor = mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, GradedColor ) );
		OutDeviceColor = LinearToST2084( OutputGamutColor );
	}
	// Linear HDR, including all color correction, but no tone curve
	else BRANCH if( GetOutputDevice() == TONEMAPPER_OUTPUT_NoToneCurve)
	{
		float3 GradedColor = ComputeGradedColor(ColorAP1);
		OutDeviceColor = GradedColor;
	}
	// "Linear" including all color correction and the tone curve, but no device gamma
	else BRANCH if (GetOutputDevice() == TONEMAPPER_OUTPUT_WithToneCurve)
	{
		OutDeviceColor = ApplyToneCurve(ColorAP1, LinearToNitsScale, AP1_2_Output, INVEOTF_LINEAR);
	}

	// OutputDevice == TONEMAPPER_OUTPUT_ExplicitGammaMapping
	// Gamma 2.2, user specified gamut
	else
	{
		float3 OutputGamutColor = ApplyToneCurve(ColorAP1, LinearToNitsScale, AP1_2_Output, INVEOTF_LINEAR);

		// This is different than the prior "gamma" curve adjustment (but reusing the variable).
		// For displays set to a gamma colorspace.
		// Note, MacOSX native output is raw gamma 2.2 not sRGB!
		OutDeviceColor = pow( OutputGamutColor, InverseGamma.z );
	}

	// Some functions output values slightly above 1. Undo this squishing in PostProcessTonemap.usf
	OutColor.rgb = OutDeviceColor * LUTInvMax;
	OutColor.a = 0;

	return OutColor;
}

float InvLUTSizeMinusOne;

RWTexture3D<float4> RWOutputTexture;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID) 
{
	float3 LUTEncodedColor = (float3)DispatchThreadId * InvLUTSizeMinusOne;
	float4 OutColor = CombineLUTsCommon(LUTEncodedColor);

	uint3 PixelPos = DispatchThreadId;
	RWOutputTexture[PixelPos] = OutColor;
}
