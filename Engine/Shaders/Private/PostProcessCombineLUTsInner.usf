// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "TonemapCommon.ush"

// ---------------------------------------------------

float3 LimitingRgbToXYZ0;
float3 LimitingRgbToXYZ1;
float3 LimitingRgbToXYZ2;
float3x3 GetLimitingRgbToXYZ()
{
	return float3x3(LimitingRgbToXYZ0, LimitingRgbToXYZ1, LimitingRgbToXYZ2);
}

float3 LimitingXYZToRgb0;
float3 LimitingXYZToRgb1;
float3 LimitingXYZToRgb2;
float3x3 GetLimitingXYZToRgb()
{
	return float3x3(LimitingXYZToRgb0, LimitingXYZToRgb1, LimitingXYZToRgb2);
}

float4 ACESMinMaxData;
float4 ACESMidData;
float4 ACESCoefsLow_0;
float4 ACESCoefsHigh_0;
float ACESCoefsLow_4;
float ACESCoefsHigh_4;
float ACESSceneColorMultiplier;
float ACESGamutCompression;

// ACES 2.0
Texture2D<float> ACESReachTable;
Texture2D<float> ACESGamutTable;
Texture2D<float> ACESGammaTable;
Texture2D<float> ACESReachTableSDR;
Texture2D<float> ACESGamutTableSDR;
Texture2D<float> ACESGammaTableSDR;
uint bScaleWhite;
uint bScaleWhiteSDR;

uint OutputDevice;
uint OutputGamut;
float OutputMaxLuminance;

float TotalSceneColorMultiplier;

uint GetOutputDevice()
{
#if OUTPUT_DEVICE_SRGB
	return TONEMAPPER_OUTPUT_sRGB;
#else
	return OutputDevice;
#endif
}

float3 ComputeOutputACES(
	float3 GradedColor,
	float3x3 LimitingRgbToXYZ,
	float3x3 LimitingXYZToRgb,
	float3x3 AP1_2_Output)
{
#if USE_ACES_2
	float3 ODTColor = ACESOutputTransform(
		GradedColor * TotalSceneColorMultiplier,
		(float3x3)WorkingColorSpace.ToAP0,
		OutputMaxLuminance,
		ACESReachTable,
		ACESGamutTable,
		ACESGammaTable,
		bScaleWhite,
		LimitingRgbToXYZ,
		LimitingXYZToRgb,
		GetColorSpace(OutputGamut));
	return ODTColor;
#else
	FACESTonemapParams AcesParams = ComputeACESTonemapParams(ACESMinMaxData, ACESMidData, ACESCoefsLow_0, ACESCoefsHigh_0, ACESCoefsLow_4, ACESCoefsHigh_4, ACESSceneColorMultiplier, ACESGamutCompression);
	float3 ODTColor = ACESOutputTransform( GradedColor, (float3x3)WorkingColorSpace.ToAP0, AcesParams);
	float3 OutDeviceColor = mul(AP1_2_Output, ODTColor);
	return OutDeviceColor;
#endif
}

float3 ComputeInvOutputACES(float3 ODTColor, float3x3 LimitingRgbToXYZ, float3x3 LimitingXYZToRgb)
{
#if USE_ACES_2
	float3 GradedColorMultiplied = ACESInvOutputTransform(
		ODTColor,
		(float3x3)WorkingColorSpace.FromAP0,
		LinearToNitsScale,
		ACESReachTableSDR,
		ACESGamutTableSDR,
		ACESGammaTableSDR,
		bScaleWhiteSDR,
		LimitingRgbToXYZ,
		LimitingXYZToRgb,
		GetColorSpace(OutputGamut));
	return GradedColorMultiplied / ACESSceneColorMultiplier;
#else
	// Magenta makes it obvious this isn't implemented.
	return float3(1., 0., 1.);
#endif
}

float4 MethodLUTsCommon(float3 LUTEncodedColor)
{
	float3 GradedColor;
	BRANCH
	if (TonemappingMethod == TONEMAPPING_METHOD_ACES)
	{
		float3 GradedColorAP1 = ST2084ToLinear(LUTEncodedColor) * LinearToNitsScaleInverse;
		GradedColor = mul( (float3x3)WorkingColorSpace.FromAP1, GradedColorAP1 );
	}
	else // TonemappingMethod == TONEMAPPING_METHOD_Filmic
	{
		// "Automatic HDR" using ACES to remap filmic.
		// Undo multiplication performed by outer LUT.
		float3 FilmNits = ST2084ToLinear(LUTEncodedColor * .5080784215);
		GradedColor = ComputeInvOutputACES(FilmNits, (float3x3)WorkingColorSpace.ToXYZ, (float3x3)WorkingColorSpace.FromXYZ);
	}

	const float3x3 AP1_2_Output = OutputGamutMappingMatrix(OutputGamut);
	float3 OutputGamutColor = ComputeOutputACES(GradedColor, GetLimitingRgbToXYZ(), GetLimitingXYZToRgb(), AP1_2_Output);

	BRANCH
	if (OutputMaxLuminance > LinearToNitsScale)
	{
		OutputGamutColor = clamp(OutputGamutColor, 0., UE_PQ_WHITE_NITS);
		OutputGamutColor = LinearToST2084(OutputGamutColor);
	}
	else
	{
		OutputGamutColor = saturate(OutputGamutColor * LinearToNitsScaleInverse);
		OutputGamutColor = LinearToSrgb(OutputGamutColor);
	}

	return float4(OutputGamutColor, 0.);
}

float InvLUTSizeMinusOne;

RWTexture3D<float4> RWOutputTexture;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID) 
{
	float3 LUTEncodedColor = (float3)DispatchThreadId * InvLUTSizeMinusOne;
	float4 OutColor = MethodLUTsCommon(LUTEncodedColor);

	uint3 PixelPos = DispatchThreadId;
	RWOutputTexture[PixelPos] = OutColor;
}
