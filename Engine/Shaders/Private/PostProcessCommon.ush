// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessCommon.usf: PostProcessing shared functions and structures.
=============================================================================*/

#pragma once

#include "SceneTexturesCommon.ush"

/// e.g.
//   PostprocessInput0.SampleLevel(PostprocessInput0Sampler, float2 UV, 0)
//   Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, float2 UV)
//   PostprocessInput0.Load(Texel.xyz)
//   PostprocessInput0.GetDimensions(uint, out uint width, out uint height, out uint levels)
//   PostprocessInput0.GetDimensions(uint, out float width, out float height, out float levels)
//   PostprocessInput0.GetDimensions(out uint width, out uint height)
//   PostprocessInput0.GetDimensions(out float width, out float height)
Texture2D PostprocessInput0;
SamplerState PostprocessInput0Sampler;
Texture2D PostprocessInput1;
SamplerState PostprocessInput1Sampler;
Texture2D PostprocessInput2;
SamplerState PostprocessInput2Sampler;
Texture2D PostprocessInput3;
SamplerState PostprocessInput3Sampler;
Texture2D PostprocessInput4;
SamplerState PostprocessInput4Sampler;
Texture2D PostprocessInput5;
SamplerState PostprocessInput5Sampler;
Texture2D PostprocessInput6;
SamplerState PostprocessInput6Sampler;
Texture2D PostprocessInput7;
SamplerState PostprocessInput7Sampler;
Texture2D PostprocessInput8;
SamplerState PostprocessInput8Sampler;
Texture2D PostprocessInput9;
SamplerState PostprocessInput9Sampler;
Texture2D PostprocessInput10;
SamplerState PostprocessInput10Sampler;

// width, height, 1/width, 1/height
float4 PostprocessInput0Size;
float4 PostprocessInput1Size;
float4 PostprocessInput2Size;
float4 PostprocessInput3Size;
float4 PostprocessInput4Size;
float4 PostprocessInput5Size;
float4 PostprocessInput6Size;
float4 PostprocessInput7Size;
float4 PostprocessInput8Size;
float4 PostprocessInput9Size;
float4 PostprocessInput10Size;

// xy = min valid BufferUV for a bilinear sampler in PostprocessInput%d, zw = max valid BufferUV for a bilinear sampler.
float4 PostprocessInput0MinMax;
float4 PostprocessInput1MinMax;
float4 PostprocessInput2MinMax;
float4 PostprocessInput3MinMax;
float4 PostprocessInput4MinMax;
float4 PostprocessInput5MinMax;
float4 PostprocessInput6MinMax;
float4 PostprocessInput7MinMax;
float4 PostprocessInput8MinMax;
float4 PostprocessInput9MinMax;
float4 PostprocessInput10MinMax;

// viewport width, height, 1/width, 1/height (scaled to the current rendertarget resolution), depends on SetViewportAndCallRHI() call
float4 ViewportSize;
// in pixels (scaled to the current rendertarget resolution), float4(minx,miny,maxx,maxy), depends on SetViewportAndCallRHI() call
uint4 ViewportRect;

// transforms ScreenPos to the screen pixel position, not the viewport local position (*xy +wz)
// (0,0) left top, (width-1,height-1) at right bottom, in pixels (without offset to reach the pixel center)
float4 ScreenPosToPixel;

float4 SceneColorBufferUVViewport;

// transform viewport UV to scene color buffer UV. As oposed to ViewportUVToBufferUV(),
// this handle viewrect change in middle of post processing caused by TAA upsample.
float2 ViewportUVToPostProcessingSceneColorBufferUV(float2 ViewportUV)
{
	return ViewportUV * SceneColorBufferUVViewport.xy + SceneColorBufferUVViewport.zw;
}

// shape of the lens for vignette like effects and masking
// @param ScreenPos -1 .. 1
// @return 0..1 0:borders, 1:center
float DiscMask(float2 ScreenPos)
{
	float x = saturate(1.0f - dot(ScreenPos, ScreenPos));

	return x * x;
}

// black on the borders
// @param ScreenPos -1 .. 1
// @return 0..1 0:borders, 1:center
float RectMask(float2 ScreenPos)
{
	float2 UV = saturate(ScreenPos * 0.5 + 0.5f);
	float2 Mask2 = UV * (1 - UV);

	return Mask2.x * Mask2.y * 8.0f;
}

// for rectangles with border
// @return >=0, 0 if inside
float ComputeDistanceToRect(int2 Pos, int2 LeftTop, int2 Extent, bool bRoundBorders = true)
{
	int2 RightBottom = LeftTop + Extent - 1;

	// use int for more precision
	int2 Rel = max(int2(0, 0), Pos - RightBottom) + max(int2(0, 0), LeftTop - Pos);

	if(bRoundBorders)
	{
		// euclidian distance (round corners)
		return length((float2)Rel);
	}
	else
	{
		// manhatten distance (90 degree corners)
		return max(Rel.x, Rel.y);
	}
}


float3 MappingPolynomial;			 // RGB = a, b, c where y = a * x*x + b * x + c

// @param InLDRColor needs to be LDR (0..1) and in linear space
half3 ColorCorrection(half3 InLDRColor)
{
	// final color correction to adjust for hardware differences, to make quick adjustements before a demo or simply a user setting
	return MappingPolynomial.x * (InLDRColor * InLDRColor) + MappingPolynomial.y * InLDRColor + MappingPolynomial.z;
}

// See EVignetteType
#define VIGNETTE_TYPE_COSINE_FOURTH_LAW (0)
#define VIGNETTE_TYPE_TEXTURE			(1)
#define VIGNETTE_TYPE_OVAL				(2)

void PackVignetteParameters(out float4 OutVignetteParams0, out float4 OutVignetteParams1, out float OutVignetteParams2, float Intensity)
{
	float3 Color	= float3(1.0, 1.0, 1.0);
	float2 Center	= float2(0.5, 0.5);
	float2 InvSize	= float2(1.0, 1.0);
	float Softness	= 1.0;

	OutVignetteParams0.x	= Intensity;
	OutVignetteParams0.yzw	= Color;
	OutVignetteParams1.xy	= Center;
	OutVignetteParams1.zw	= InvSize;
	OutVignetteParams2		= Softness;
}

// Generate a mask to darken the screen borders.
// (Camera artifact and artistic effect)
// @param VignetteCircleSpacePos from VignetteSpace()
float3 ComputeVignetteMask(
	float2 VignetteCircleSpacePos, 
	uint VignetteType, 
	float4 VignetteParams0, 
	float4 VignetteParams1, 
	float  VignetteParams2
#if USE_VIGNETTE_TEXTURE
	, Texture2D VignetteTexture
	, SamplerState TextureSampler
#endif
)
{
	float Intensity = VignetteParams0.x;
	float3 Color	= VignetteParams0.yzw;
	float2 Center	= VignetteParams1.xy - 0.5;
	float2 InvSize	= VignetteParams1.zw;
	float Softness	= VignetteParams2;

#if !USE_ADVANCED_VIGNETTE
	VignetteType = VIGNETTE_TYPE_COSINE_FOURTH_LAW; // force vignette type
#endif

	float Weight = 1.0;
	BRANCH
	if (Intensity <= 0)
	{ }
	else if (VignetteType == VIGNETTE_TYPE_COSINE_FOURTH_LAW)
	{
		VignetteCircleSpacePos -= Center;
		VignetteCircleSpacePos *= Intensity;
		float Tan2Angle = dot(VignetteCircleSpacePos, VignetteCircleSpacePos);
		float Cos4Angle = Square(rcp(Tan2Angle + 1));
		Weight = Cos4Angle;
	}
#if USE_VIGNETTE_TEXTURE
	else if (VignetteType == VIGNETTE_TYPE_TEXTURE)
	{
		Weight = VignetteTexture.SampleLevel(TextureSampler, (InvSize * VignetteCircleSpacePos - Center) * 0.5 + 0.5, 0).r;
		Weight = lerp(1.0, Weight, Intensity);
	}
#endif
#if USE_ADVANCED_VIGNETTE
	else
	{
		float2 Position = InvSize * VignetteCircleSpacePos;

		float Distance = 0;
		if (VignetteType == VIGNETTE_TYPE_OVAL)
		{
			Distance = length2(Position - Center);
		}

		Weight = pow(saturate(1.0 - Distance), Softness);
		Weight = lerp(1.0, Weight, Intensity);
	}
#endif

	return lerp(Color, (float3)1, Weight);
}

// Scale {-1 to 1} of the viewport space to vignette circle space.
// Vignette space is scaled such that regardless of viewport aspect ratio, 
// corners are at the same brightness on a circle.
float2 VignetteSpace(float2 Pos, uint VignetteType, float AspectRatio)
{
	if (VignetteType == VIGNETTE_TYPE_TEXTURE)
	{
		float InvAspectRatio = rcp(AspectRatio);
		return Pos * float2(saturate(InvAspectRatio), saturate(AspectRatio));
	}
	else
	{
		float Scale = sqrt(2.0) / sqrt(1.0 + AspectRatio * AspectRatio);
		return Pos * float2(1.0, AspectRatio) * Scale;
	}
}

float2 VignetteSpace(float2 Pos, uint VignetteType)
{
	return VignetteSpace(Pos, VignetteType, ViewportSize.y * ViewportSize.z);
}

half4 UnwrappedTexture3DSample( Texture2D Texture, SamplerState Sampler, float3 UVW, float Size, float InvSize )
{
	// a volume texture 16x16x16 would be unwrapped to a 2d texture 256x16

	float IntW = floor( UVW.z * Size - 0.5 );
	half FracW = UVW.z * Size - 0.5 - IntW;

	float U = ( UVW.x + IntW ) * InvSize;
	float V = UVW.y;

	half4 RG0 = Texture2DSample( Texture, Sampler, float2(U, V) );
	half4 RG1 = Texture2DSample( Texture, Sampler, float2(U + InvSize, V) );

	return lerp(RG0, RG1, FracW);
}

bool IsComputeUVOutOfBounds(in float2 UV)
{
	float2 CenterDist = abs(UV - 0.5f);
	return (max(CenterDist.x, CenterDist.y) >= 0.5f);
}

// Constants for DOF blend in.
half CocMaxRadiusInPixelsRcp()
{
	half2 MaxOffset = half2(-2.125, -0.50) * 2.0;
	return rcp(sqrt(dot(MaxOffset, MaxOffset)));
}

half2 CocBlendScaleBias()
{
	half Start = 0.25 * CocMaxRadiusInPixelsRcp();
	half End = 1.0 * CocMaxRadiusInPixelsRcp();
	half2 ScaleBias;
	ScaleBias.x = 1.0 / (End - Start);
	ScaleBias.y = (-Start) * ScaleBias.x;
	return ScaleBias;
}

half2 CocBlendScaleBiasFine()
{
	half Start = 0.0 * CocMaxRadiusInPixelsRcp();
	half End = 0.5 * CocMaxRadiusInPixelsRcp();
	half2 ScaleBias;
	ScaleBias.x = 1.0 / (End - Start);
	ScaleBias.y = (-Start) * ScaleBias.x;
	return ScaleBias;
}

