// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"
#include "LensDistortion.ush"

#ifndef MSAA_SAMPLE_COUNT
#define MSAA_SAMPLE_COUNT 1
#endif

#ifndef EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING
#define EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING 0
#endif


#define CONFIG_SAMPLES 6

// K = Center of the nearest input pixel.
// O = Center of the output pixel.
//
//          |           |
//    0     |     1     |     2
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          | O         |
//    3     |     K     |     5
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          |           |
//    6     |     7     |     8
//          |           |
//
static const int2 kOffsets3x3[9] =
{
	int2(-1, -1),
	int2(0, -1),
	int2(1, -1),
	int2(-1,  0),
	int2(0,  0), // K
	int2(1,  0),
	int2(-1,  1),
	int2(0,  1),
	int2(1,  1),
};

// T = Center of the nearest top left pixel input pixel.
// O = Center of the output pixel.
//
//          | 
//    T     |     .
//          | 
//       O  | 
//  --------+--------
//          | 
//          | 
//    .     |     .
//          | 
static const int2 Offsets2x2[4] =
{
	int2( 0,  0), // T
	int2( 1,  0),
	int2( 0,  1),
	int2( 1,  1),
};

// Indexes of the 3x3 square.
static const uint kSquareIndexes3x3[9] = { 4u, 0u, 1u, 2u, 3u, 5u, 6u, 7u, 8u };

// Indexes of the offsets to have plus + shape.
static const uint kPlusIndexes3x3[5] = { 4u, 1u, 3u, 5u, 7u };

Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float4, MSAA_SAMPLE_COUNT> EditorPrimitivesColor;
Texture2DMS<float, MSAA_SAMPLE_COUNT> EditorPrimitivesDepth;
Texture2DMS<uint2, MSAA_SAMPLE_COUNT> EditorPrimitivesStencil;
#else
Texture2D EditorPrimitivesColor;
Texture2D<float> EditorPrimitivesDepth;
Texture2D<uint2> EditorPrimitivesStencil;
#endif

Texture2D ColorTexture;
SamplerState ColorSampler;

Texture2D DepthTexture;
SamplerState DepthSampler;

Texture2D VelocityTexture;
SamplerState VelocitySampler;

Texture2D PrevHistoryTexture;
SamplerState PrevHistorySampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Depth)
SCREEN_PASS_TEXTURE_VIEWPORT(PrevHistory)
SCREEN_PASS_TEXTURE_VIEWPORT(History)
SCREEN_PASS_TEXTURE_VIEWPORT(EditorPrimitives)

FScreenTransform PassSvPositionToViewportUV;
FScreenTransform ViewportUVToColorUV;
FScreenTransform ViewportUVToDepthUV;
FScreenTransform ViewportUVToEditorPrimitivesUV;

float2 DepthTextureJitter;
uint bCameraCut;
uint bProcessAlpha;

float OccludedDitheringPatternSize;
float OccludedDithering;
float OccludedBrightness;

float4 SampleOffsetArray[MSAA_SAMPLE_COUNT];

uint bOpaqueEditorGizmo;
uint bCompositeAnyNonNullDepth;

void MainTemporalUpsampleDepthPS(
	float4 SvPosition : SV_POSITION,
	out float OutDeviceZ : SV_Target0)
{
	float2 ViewportUV = (SvPosition.xy - History_ViewportMin) * History_ViewportSizeInverse;
	float2 ScreenPos = ViewportUVToScreenPos(ViewportUV);
	
	// Pixel coordinate of the center of output pixel O in the input viewport.
	float2 PPCo = ViewportUV * Depth_ViewportSize + DepthTextureJitter;

	// Pixel coordinate of the center of the nearest input pixel K.
	float2 PPCk = floor(PPCo) + 0.5;
	
	// Sample nearest depth
	float InputDeviceZ;
	{
		float2 SampleInputBufferUV = (Depth_ViewportMin + PPCk) * Depth_ExtentInverse;
		SampleInputBufferUV = clamp(SampleInputBufferUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
			
		InputDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, SampleInputBufferUV, 0).r;
	}

	// Compute the screen position to sample in history
	float2 PrevScreenPos = ScreenPos;
	#if 1
	{
		float4 ThisClip = float4(ScreenPos, InputDeviceZ, 1);
		float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);

		PrevScreenPos = PrevClip.xy / PrevClip.w;

		#if 0
		{
			float2 SampleInputBufferUV = (Depth_ViewportMin + PPCk) * Depth_ExtentInverse;
			SampleInputBufferUV = clamp(SampleInputBufferUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
			
			float4 EncodedVelocity = VelocityTexture.SampleLevel(VelocitySampler, SampleInputBufferUV, 0);
			bool DynamicN = EncodedVelocity.x > 0.0;
			if(DynamicN)
			{
				PrevScreenPos = ScreenPos - DecodeVelocityFromTexture(EncodedVelocity).xy;
			}
		}
		#endif
	}
	#endif

	// Sample the history.
	float PrevHistoryDeviceZ;
	{
		float2 PrevHistoryBufferUV = (PrevHistory_ScreenPosToViewportScale * PrevScreenPos + PrevHistory_ScreenPosToViewportBias) * PrevHistory_ExtentInverse;
		PrevHistoryBufferUV = clamp(PrevHistoryBufferUV, PrevHistory_UVViewportBilinearMin, PrevHistory_UVViewportBilinearMax);

		PrevHistoryDeviceZ = Texture2DSampleLevel(PrevHistoryTexture, PrevHistorySampler, PrevHistoryBufferUV, 0).r;
	}

	// Correct the DeviceZ from the previous depth to the current depth.
	float CorrectedPrevHistoryDeviceZ;
	#if 1
	{
		float PrevHistoryDepth = ConvertFromDeviceZ(PrevHistoryDeviceZ);

		const float3 PreViewTranslationOffset = DFFastSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
		const float3 PrevHistoryTranslatedWorldPosition = mul(float4(PrevScreenPos * PrevHistoryDepth, PrevHistoryDepth, 1), View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;

		float CorrectedPrevHistoryDepth = mul(float4(PrevHistoryTranslatedWorldPosition, 1.0), View.TranslatedWorldToView).z;
		
		CorrectedPrevHistoryDeviceZ = ConvertToDeviceZ(CorrectedPrevHistoryDepth);
	}
	#else
	{
		CorrectedPrevHistoryDeviceZ = PrevHistoryDeviceZ;
	}
	#endif

	// Replaces NaN from history with 0.0
	CorrectedPrevHistoryDeviceZ = -min(-CorrectedPrevHistoryDeviceZ, 0.0);

	// Sample current frame
	bool bUpdateHistory;
	float InputMinDeviceZ;
	float InputMaxDeviceZ;
	{
		// Vector in pixel between pixel K -> O.
		float2 dKO = float2(PPCo - PPCk);
		
		bUpdateHistory = all(abs(dKO) < 0.5 * (Depth_ViewportSize.x * History_ViewportSizeInverse.x)) || bCameraCut != 0;

		InputMinDeviceZ = InputDeviceZ;
		InputMaxDeviceZ = InputDeviceZ;

		UNROLL_N(CONFIG_SAMPLES - 1)
		for (uint SampleId = 1; SampleId < CONFIG_SAMPLES; SampleId++)
		{
			float2 PixelOffset;
			
			#if CONFIG_SAMPLES == 9
			{
				const uint SampleIdx = kSquareIndexes3x3[SampleId];
				PixelOffset = kOffsets3x3[SampleIdx];
			}
			#elif CONFIG_SAMPLES == 5 || CONFIG_SAMPLES == 6
			{
				if (SampleId == 5)
				{
					PixelOffset = SignFastInt(float2(dKO));
				}
				else
				{
					const uint SampleIdx = kPlusIndexes3x3[SampleId];
					PixelOffset = kOffsets3x3[SampleIdx];
				}
			}
			#else
				#error Unknown sample count
			#endif
			
			float2 SampleInputBufferUV = (Depth_ViewportMin + PPCk + PixelOffset) * Depth_ExtentInverse;
			SampleInputBufferUV = clamp(SampleInputBufferUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);

			float SampleDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, SampleInputBufferUV, 0).r;
			
			InputMinDeviceZ = min(InputMinDeviceZ, SampleDeviceZ);
			InputMaxDeviceZ = max(InputMaxDeviceZ, SampleDeviceZ);
		}
	}

	// Reject history based on neighborhood
	float ClampedHistoryDeviceZ = clamp(CorrectedPrevHistoryDeviceZ, InputMinDeviceZ, InputMaxDeviceZ);

	// Output history
	float FinalHistoryDeviceZ = bUpdateHistory ? InputDeviceZ : ClampedHistoryDeviceZ;
	
	OutDeviceZ = -min(-FinalHistoryDeviceZ, 0.0);
}

#ifdef MainPopulateSceneDepthPS

float4 OutputViewRectMinAndInvSize;

void MainPopulateSceneDepthPS(
	#if USE_MSAA && !COMPILER_HLSLCC
		sample noperspective float4 SvPosition : SV_POSITION,
	#else
		noperspective float4 SvPosition : SV_POSITION,
	#endif
	out float4 OutColor : SV_Target0,
	out float OutDepth : SV_DEPTH)
{
	float2 ViewportUV = (SvPosition.xy - OutputViewRectMinAndInvSize.xy) * OutputViewRectMinAndInvSize.zw;
	
	float2 DepthUV = clamp(
		(Depth_ViewportMin + ViewportUV * Depth_ViewportSize - DepthTextureJitter) * Depth_ExtentInverse,
		Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
	DepthUV = clamp(DepthUV, float2(0, 0), float2(1, 1));

#if FORCE_DEBUG_DRAW_COLOR
	//This extended pass is primarily to compensate for the lack of scene color output when compositing the debug draw primitives with SimpleElementShaders
	//Draw out the color whilst compensating for the viewport not necessarily starting at 0,0. 
	//Does not use jitter for Scene Color as it is not upscaled for this pass
	float2 ColorUV = clamp(
		(Color_ViewportMin + ViewportUV * Color_ViewportSize) * Color_ExtentInverse,
		Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);
	ColorUV = clamp(ColorUV, float2(0, 0), float2(1, 1));
	OutColor = Texture2DSampleLevel(ColorTexture, ColorSampler, ColorUV, 0).rgba;

#if METAL_MSAA_HDR_DECODE
	OutColor.rgb *= rcp(OutColor.r * (-0.299) + OutColor.g * (-0.587) + OutColor.b * (-0.114) + 1.0);
#endif

#else
	OutColor = 0.0;
#endif

	OutDepth = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV, 0).r;
}

#endif

float ComputeDepthMask(float SceneDeviceZ, float EditorDeviceZ)
{
	// Soft Bias with SceneDeviceZ for best quality
	const float DeviceDepthFade = 0.00005f;

	return saturate(1.0f - (SceneDeviceZ - EditorDeviceZ) / DeviceDepthFade);
}

void ResolveEditorPrimitiveColor(int2 PixelPos, out float4 OutColor, out float OutDeviceZ)
{
	// Furthest device Z or 0 if there is none.
	OutDeviceZ = 0;
	OutColor = 0;
	

	// Bring out of premultiplied.
	OutColor.rgb /= max(OutColor.a, 0.0001f);
	// Fix gamma.
	OutColor.rgb = pow(OutColor.rgb, 1.0f / 2.2f);
	// Bring back to premultiplied
	OutColor.rgb *= OutColor.a;
}

void MainCompositePostProcessPrimitivesPS(
	float4 SvPosition : SV_POSITION,
#if MSAA_DONT_RESOLVE && MSAA_SAMPLE_COUNT > 1
	uint TargetSampleIndex : SV_SampleIndex,
#endif
	out float4 OutColor : SV_Target0
 #if WRITE_DEPTH
	,out float OutDepth : SV_Depth
 #endif
    )
{
	const float2 DistortedViewportUV = ApplyScreenTransform(SvPosition.xy, PassSvPositionToViewportUV);
	const float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);

	const float2 DistortedColorUV = ApplyScreenTransform(DistortedViewportUV, ViewportUVToColorUV);
	const float2 ColorUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToColorUV);
	const float2 DepthUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToDepthUV);
	const float2 EditorPrimitivesUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToEditorPrimitivesUV);

	const int2 EditorPrimitivesPixelPos = int2(EditorPrimitivesUV * EditorPrimitives_Extent);
	
	float4 SceneColor = Texture2DSample(ColorTexture, ColorSampler, DistortedColorUV);

	// Resolve editor primitive scene color and depth.
	float4 EditorPrimitiveColor;
	float DepthMask;
	#if WRITE_DEPTH
	OutDepth = 0;
    #endif
	#if MSAA_DONT_RESOLVE && MSAA_SAMPLE_COUNT > 1
	{
		TargetSampleIndex = min(TargetSampleIndex, MSAA_SAMPLE_COUNT-1);
		const float2 SampleOffset = SampleOffsetArray[TargetSampleIndex].xy;

		float2 SampleDepthUV = (UndistortedViewportUV * Depth_ViewportSize + Depth_ViewportMin + SampleOffset - DepthTextureJitter) * Depth_ExtentInverse;
		SampleDepthUV = clamp(SampleDepthUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);

		EditorPrimitiveColor = EditorPrimitivesColor.Load(EditorPrimitivesPixelPos, TargetSampleIndex);
		float SampleEditorDeviceZ = EditorPrimitivesDepth.Load(EditorPrimitivesPixelPos, TargetSampleIndex).r;
		uint SampleEditorStencil = EditorPrimitivesStencil.Load(EditorPrimitivesPixelPos, TargetSampleIndex).g;
		float SampleSceneDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, SampleDepthUV, 0).r;

		bool bIsForegroundNoDitheringSample = SampleEditorStencil & EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING;
		if (bIsForegroundNoDitheringSample)
		{
			SampleEditorDeviceZ = NearDepthValue;
		}

		if (SampleEditorDeviceZ > 0.0 && bCompositeAnyNonNullDepth)
		{
			EditorPrimitiveColor.a = 1;
		}

		DepthMask = ComputeDepthMask(SampleSceneDeviceZ, SampleEditorDeviceZ);
		#if WRITE_DEPTH
		OutDepth = max(SampleSceneDeviceZ, SampleEditorDeviceZ);
        #endif
	}
	#elif MSAA_SAMPLE_COUNT > 1
	{
		EditorPrimitiveColor = 0.0;
		DepthMask = 0.0;
		float DepthMaskWeight = 0.0;

		UNROLL_N(MSAA_SAMPLE_COUNT)
		for (uint SampleIndex = 0; SampleIndex < MSAA_SAMPLE_COUNT; ++SampleIndex)
		{
			const float2 SampleOffset = SampleOffsetArray[SampleIndex].xy;

			float2 SampleDepthUV = (UndistortedViewportUV * Depth_ViewportSize + Depth_ViewportMin + SampleOffset - DepthTextureJitter) * Depth_ExtentInverse;
			SampleDepthUV = clamp(SampleDepthUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);
			float SampleSceneDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, SampleDepthUV, 0).r;

			float4 Sample = EditorPrimitivesColor.Load(EditorPrimitivesPixelPos, SampleIndex);
			float SampleEditorDeviceZ = EditorPrimitivesDepth.Load(EditorPrimitivesPixelPos, SampleIndex).r;
			uint SampleEditorStencil = EditorPrimitivesStencil.Load(EditorPrimitivesPixelPos, SampleIndex).g;

			bool bIsForegroundNoDitheringSample = SampleEditorStencil & EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING;
			if (bIsForegroundNoDitheringSample)
			{
				SampleEditorDeviceZ = NearDepthValue;
			}

			// Check if any color was applied to this pixel. Note: This prevents actual black pixels from being visible.
			float Weight = Sample.a;

			float SampleDepthMask = ComputeDepthMask(SampleSceneDeviceZ, SampleEditorDeviceZ);

			if (SampleEditorDeviceZ > 0.0 && bCompositeAnyNonNullDepth)
			{
				Weight = 1;
			}

			FLATTEN
			if (Weight)
			{
				EditorPrimitiveColor += float4(Sample.rgb, Weight);

				DepthMask += SampleDepthMask * Weight;
				DepthMaskWeight += Weight;
			}

			#if WRITE_DEPTH
			OutDepth = max3(OutDepth, SampleSceneDeviceZ, SampleEditorDeviceZ);
            #endif
		}

		EditorPrimitiveColor.rgb /= MSAA_SAMPLE_COUNT;
		EditorPrimitiveColor.a /= MSAA_SAMPLE_COUNT;
		
		DepthMask *= DepthMaskWeight > 0 ? rcp(DepthMaskWeight) : 0.0;
	}
	#else
	{
		// De-jitter the sample position and make a filtered lookup - for planes this allows to reconstruct a much less jittery depth comparison function. It doesn't fix silhouettes, however.
		float2 SampleDepthUV = (UndistortedViewportUV * Depth_ViewportSize + Depth_ViewportMin - DepthTextureJitter) * Depth_ExtentInverse;
		SampleDepthUV = clamp(SampleDepthUV, Depth_UVViewportBilinearMin, Depth_UVViewportBilinearMax);

		float SceneDeviceZ = Texture2DSampleLevel(DepthTexture, DepthSampler, SampleDepthUV, 0).r;
		float EditorDeviceZ = EditorPrimitivesDepth.Load(int3(EditorPrimitivesPixelPos, 0)).r;
		uint EditorStencil = EditorPrimitivesStencil.Load(int3(EditorPrimitivesPixelPos, 0)).g;

		bool bIsForegroundNoDitheringSample = EditorStencil & EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING;
		if (bIsForegroundNoDitheringSample)
		{
			EditorDeviceZ = NearDepthValue;
		}
		
		EditorPrimitiveColor = EditorPrimitivesColor.Load(int3(EditorPrimitivesPixelPos, 0));
		DepthMask = ComputeDepthMask(SceneDeviceZ, EditorDeviceZ);
		#if WRITE_DEPTH
		OutDepth = HAS_INVERTED_Z_BUFFER ? max(SceneDeviceZ, EditorDeviceZ) : min(SceneDeviceZ, EditorDeviceZ);
        #endif

		if (EditorDeviceZ > 0.0 && bCompositeAnyNonNullDepth)
		{
			EditorPrimitiveColor.a = 1;
		}
	}
	#endif
	
	// Fixes the gama of editor primitives
	{
		// Bring out of premultiplied.
		EditorPrimitiveColor.rgb /= max(EditorPrimitiveColor.a, 0.0001f);
		// Fix gamma.
		EditorPrimitiveColor.rgb = pow(EditorPrimitiveColor.rgb, 1.0f / 2.2f);
		// Bring back to premultiplied
		EditorPrimitiveColor.rgb *= EditorPrimitiveColor.a;
	}

	// Generate NxN square tiling pattern for foreground primitive that end up behind scene opaque primitives.
	float PatternMask = fmod(
		floor(EditorPrimitivesPixelPos.x / OccludedDitheringPatternSize)
		+ floor(EditorPrimitivesPixelPos.y / OccludedDitheringPatternSize),
		2.0f) * OccludedBrightness;

	// the contants express the two opacity values we see when the primitive is hidden
	float LowContrastPatternMask = lerp(1.0 - OccludedDithering, 1.0f, PatternMask);

	LowContrastPatternMask = saturate(lerp(LowContrastPatternMask, 1, bOpaqueEditorGizmo));

	float HiddenMask = lerp(OccludedBrightness, 1.0f, DepthMask);
	float DarkenMask = lerp(LowContrastPatternMask, 1.0f, DepthMask);

	// Blend editor primitives with scene color.
	OutColor.rgb = SceneColor.rgb * (1 - EditorPrimitiveColor.a) + EditorPrimitiveColor.rgb * (DarkenMask * HiddenMask);

	FLATTEN
	if (bProcessAlpha)
	{
		OutColor.a = lerp(SceneColor.a, 1, EditorPrimitiveColor.a * DarkenMask * HiddenMask);
	}
	else
	{
		OutColor.a = 0;
	}
}
