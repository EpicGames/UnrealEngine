// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDOF.usf: PostProcessing Depth of Field
=============================================================================*/

#define FAR_BLUR			FAR_BLUR_COUNT >= 1
#define NEAR_BLUR			NEAR_BLUR_COUNT >= 1
#define DOF_VIGNETTE		NEAR_BLUR_COUNT == 2
#define MRT_COUNT			FAR_BLUR_COUNT + (NEAR_BLUR_COUNT > 0 ? 1 : 0)

#include "Common.ush"
#include "DepthOfFieldCommon.ush"

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D DofFarBlurTexture;
SamplerState DofFarBlurSampler;

Texture2D DofNearBlurTexture;
SamplerState DofNearBlurSampler;

Texture2D SunShaftAndDofTexture;
SamplerState SunShaftAndDofSampler;

float4 BufferSizeAndInvSize;

// todo move to central place
float ComputeDOFNearFocalMask(float SceneDepth)
{
	float NearFocalPlane = View.DepthOfFieldFocalDistance;

	return saturate((NearFocalPlane - SceneDepth) / View.DepthOfFieldNearTransitionRegion);
}

// todo move to central place
float ComputeDOFFarFocalMask(float SceneDepth)
{
	float FarFocalPlane = View.DepthOfFieldFocalDistance + View.DepthOfFieldFocalRegion;

	return saturate((SceneDepth - FarFocalPlane) / View.DepthOfFieldFarTransitionRegion);
}

// Calculate focal masks from Mobile's PP CoC.
float2 DecodeDOFFocalMask(float SceneDepth)
{
	float2 Ret;
	SceneDepth = min(1.0, SceneDepth);
	if (SceneDepth > 0.5)
	{
		Ret.x = (SceneDepth - 0.5) * 2.0;
		Ret.y = 0;
	}
	else
	{
		Ret.y = 1.0 - (SceneDepth * 2.0);
		Ret.x = 0;
	}
	return Ret * View.DepthOfFieldScale;
}

// @return .x:far, .y:near
float2 ComputeDOFFocalMask(float SceneDepth, float SkyWithoutHorizonMask)
{
	float2 Ret = float2(ComputeDOFFarFocalMask(SceneDepth), ComputeDOFNearFocalMask(SceneDepth));

	float SkyFocusDistance = DepthOfFieldParams.x;

	// The skybox should not be faded out, expect in the horizon, this can be optimized
	if(SceneDepth > SkyFocusDistance)
	{
		Ret.x = lerp(Ret.x, 0,  SkyWithoutHorizonMask);
	}

	Ret = DecodeDOFFocalMask(SceneDepth);

	return Ret;
}

float4 ReadFullResAndDepth(float2 UV, in out float Depth)
{
	float4 Color = Texture2DSampleLevel(SceneColorTexture, SceneColorSampler, UV, 0);
	Color.a = Texture2DSampleLevel(SunShaftAndDofTexture, SunShaftAndDofSampler, UV, 0).r;

	Depth = Color.a;

	Color.a = 1.0;
	return Color;
}

// pixel shader entry point
void SetupPS(
	noperspective float4 unused : TEXCOORD0,
	float4 SvPosition : SV_POSITION
	, out float4 OutColor0 : SV_Target0
#if MRT_COUNT > 1
	, out float4 OutColor1 : SV_Target1
#endif
	)
{
	float DoFToScreenScale = 4;
	ResolvedView = ResolveView();
	// Manually compute UV and ScreenSpacePos from SvPosition to avoid south east shiting of the pixel in bottom right corner
	// caused by the DestRect.Size() + 1 viewport size done for odd view rect resolution.
	float2 ViewportUV = min(SvPositionToViewportUV(SvPosition * DoFToScreenScale), 1 - View.ViewSizeAndInvSize.zw * 0.5);
	float2 UV = ViewportUVToBufferUV(ViewportUV) - ((0.5f - View.TemporalAAParams.zw) * BufferSizeAndInvSize.zw);
	float2 ScreenSpacePos = DoFToScreenScale * (float2(ViewportUV.x, 1 - ViewportUV.y) - 0.5);

	float2 Offset = 0.5f * BufferSizeAndInvSize.zw;

	float MaskDistance = View.DepthOfFieldFocalDistance + View.DepthOfFieldFocalRegion * 0.5f;

	// Mobile path gathers Depth later in ReadFullResAndDepth
	float4 DepthQuad = float4(0,0,0,0);

	float4 FarColor = 0;
	float4 NearColor = 0;

	float2 Mask;
	float4 Sample;

	// for each sample of the full res input image
	// we compute the mask (front of back layer)
	// and put into MRT0 or MRT1
	
	// can be optimized, needed to not blur the skybox
	float3 ScreenVector = normalize(mul(float3(ScreenSpacePos, 1), DFToFloat3x3(PrimaryView.ScreenToWorld)).xyz);
	float SkyWithoutHorizonMask = saturate(ScreenVector.z * 3.0f);

	// 0:see though..1:Near blurred
	float VignetteMask = 0;

#if DOF_VIGNETTE
	{
		float DepthOfFieldVignetteMul = DepthOfFieldParams.y;
		float DepthOfFieldVignetteAdd = DepthOfFieldParams.z;

		// todo: we could optimize that multiplication away
		float InvAspectRatio = View.ViewSizeAndInvSize.y * View.ViewSizeAndInvSize.z;

		float CenterDist = length(ScreenSpacePos * float2(1, InvAspectRatio));

		// We prepare the constants on CPU side and use MAD (Multiply and Add) as this is faster
		VignetteMask = saturate(CenterDist * DepthOfFieldVignetteMul + DepthOfFieldVignetteAdd);
	}
#endif

	Sample = ReadFullResAndDepth(UV + Offset * float2(-1, 1), DepthQuad.x);
	Mask = ComputeDOFFocalMask(DepthQuad.x, SkyWithoutHorizonMask);
	FarColor += Sample * Mask.x;
	NearColor += Sample * lerp(Mask.y, 1, VignetteMask);

	Sample = ReadFullResAndDepth(UV + Offset * float2(1, 1), DepthQuad.y);
	Mask = ComputeDOFFocalMask(DepthQuad.y, SkyWithoutHorizonMask);
	FarColor += Sample * Mask.x;
	NearColor += Sample * lerp(Mask.y, 1, VignetteMask);

	Sample = ReadFullResAndDepth(UV + Offset * float2(1, -1), DepthQuad.z);
	Mask = ComputeDOFFocalMask(DepthQuad.z, SkyWithoutHorizonMask);
	FarColor += Sample * Mask.x;
	NearColor += Sample * lerp(Mask.y, 1, VignetteMask);

	Sample = ReadFullResAndDepth(UV + Offset * float2(-1, -1), DepthQuad.w);
	Mask = ComputeDOFFocalMask(DepthQuad.w, SkyWithoutHorizonMask);
	FarColor += Sample * Mask.x;
	NearColor += Sample * lerp(Mask.y, 1, VignetteMask);

	// we average 4 samples
	FarColor /= 4;
	NearColor /= 4;

#if MRT_COUNT > 1
	OutColor0 = FarColor;
	OutColor1 = NearColor;
#else
	#if NEAR_BLUR
		OutColor0 = NearColor;
	#else
		OutColor0 = FarColor;
	#endif
#endif
}


float4 DepthOfFieldUVLimit;

// pixel shader to combine the full res scene and the blurred images behind and in front of the the focal plane
void MainRecombinePS(
	noperspective float4 unused : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	// SceneColor in full res
	float2 PixelPosCenter = SvPosition.xy;

	float2 FullResUV = PixelPosCenter * BufferSizeAndInvSize.zw;

	// DOF in half res
	float2 ViewportUV = SvPositionToBufferUV(SvPosition);

	// Clamp UV to avoid pulling bad data.
	ViewportUV.x = clamp(ViewportUV.x, DepthOfFieldUVLimit.x, DepthOfFieldUVLimit.z);
	ViewportUV.y = clamp(ViewportUV.y, DepthOfFieldUVLimit.y, DepthOfFieldUVLimit.w);

	float4 SceneColorAndDepth = Texture2DSample(SceneColorTexture, SceneColorSampler, FullResUV);
	SceneColorAndDepth.a = Texture2DSample(SunShaftAndDofTexture, SunShaftAndDofSampler, FullResUV).r;

	float3 UnfocusedSceneColor = SceneColorAndDepth.rgb;

	// I'm presuming all that matters here is the W==0 bit to mask out this value
	// TODO: Should check that compiler is doing a good job of removing the usages of this
	// from the rest of the code. It has no reason not to be able to do so...

	float4 DOFAccumLayer1 = float4(0,0,0,0); 
	float4 DOFAccumLayer3 = float4(0,0,0,0); 

#if FAR_BLUR
	DOFAccumLayer1 = Texture2DSample(DofFarBlurTexture, DofFarBlurSampler, ViewportUV);
#endif

#if NEAR_BLUR
	DOFAccumLayer3 = Texture2DSample(DofNearBlurTexture, DofNearBlurSampler, ViewportUV);
#endif

	float Layer1Mask = DOFAccumLayer1.a;
	float Layer2Mask = 1.0f - ComputeDOFFarFocalMask(SceneColorAndDepth.a);
//	float Layer2Mask = 1.0f - DOFAccumLayer1.a;
	float Layer3Mask = DOFAccumLayer3.a;
	float PerPixelNearMask = ComputeDOFNearFocalMask(SceneColorAndDepth.a);

	float2 DoFMasks = DecodeDOFFocalMask(SceneColorAndDepth.a);
	Layer2Mask = 1.0-DoFMasks.x;

	// 3 layers
	float Div0Bias = 0.0001f;

	// RGB color, A how much the full resolution showes through
	float3 LayerMerger = 0;

	// Layer 1: half res background
	LayerMerger = (UnfocusedSceneColor * Div0Bias + DOFAccumLayer1.rgb) / (DOFAccumLayer1.a + Div0Bias);

	// Needed to cope with the skybox not being blurred, the tweak value
	// avoids having a discontinuity between blurry far objects and the skybox
	// and is choosen to not produce too much blobby looking out of focus rendering.
	float Blend = DOFAccumLayer1.a;
	// Magic function to transform alpha into smooth blend function against in-focus skybox.
	Blend = sqrt(Blend);
	Blend = sqrt(Blend);
	Blend = Blend * Blend * (3.0 - 2.0 * Blend);
	LayerMerger = lerp(UnfocusedSceneColor, LayerMerger, Blend);

	// Layer 2: then we add the focused scene to fill the empty areas
	float Smash = 0.25;
	Layer2Mask = saturate((Layer2Mask - (1.0 - Smash)) * rcp(Smash));
	Layer2Mask *= Layer2Mask;
//	LayerMerger = lerp(LayerMerger, SceneColorAndDepth.rgb, Layer2Mask * (1 - PerPixelNearMask));
	LayerMerger = lerp(LayerMerger, SceneColorAndDepth.rgb, Layer2Mask);

	float3 FrontLayer = (UnfocusedSceneColor * Div0Bias + DOFAccumLayer3.rgb) / (DOFAccumLayer3.a + Div0Bias);

	// Layer 3: on top of that blend the front half res layer	
	LayerMerger = lerp(LayerMerger, FrontLayer, saturate(Layer3Mask * 5));

	OutColor.rgb = LayerMerger;
	OutColor.a = 0;
}




