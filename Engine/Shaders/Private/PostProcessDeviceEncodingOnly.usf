// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDeviceEncodingOnly.usf: PostProcessing device encoding only. 
	This replaces the tonemapper when displaying debug post process materials
	used for monitor calibration.

	Note: Any changes to device encoding logic must also be made 
	in PostProcessCombineLUTs.usf's corresponding pixel shader.

=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "TonemapCommon.ush"
#include "ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D ColorTexture;
SamplerState ColorSampler;

float EditorNITLevel;

uint OutputDevice;
uint OutputGamut;
float OutputMaxLuminance;

float TotalSceneColorMultiplier;

float4 ACESMinMaxData;
float4 ACESMidData;
float4 ACESCoefsLow_0;
float4 ACESCoefsHigh_0;
float ACESCoefsLow_4;
float ACESCoefsHigh_4;
float ACESSceneColorMultiplier;
float ACESGamutCompression;

// ACES 2.0
Texture2D<float> ACESReachTable;
Texture2D<float> ACESGamutTable;
Texture2D<float> ACESGammaTable;
uint bScaleWhite;

uint GetOutputDevice()
{
#if OUTPUT_DEVICE_SRGB
	return TONEMAPPER_OUTPUT_sRGB;
#else
	return OutputDevice;
#endif
}

float4 SampleSceneColor(float2 SceneUV)
{
	return Texture2DSample(ColorTexture, ColorSampler, SceneUV);
}

float4 DeviceEncodingOnlyCommonPS(
	float2 UV,
	float4 SvPosition
	)
{
	float4 OutColor = 0;

	float2 SceneUV = UV.xy;

	half4 SceneColor = SampleSceneColor(SceneUV);

	const float3x3 AP1_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	const float3x3 AP1_2_Output  = OutputGamutMappingMatrix( OutputGamut );

	// Apply "gamma" curve adjustment.
	float3 FilmColor = pow(max(0, SceneColor.rgb), InverseGamma.y);
	
	// Note: Any changes to device encoding logic below must also be made 
	// in PostProcessCombineLUTs.usf's corresponding pixel shader.
	half3 OutDeviceColor = 0;
	BRANCH

	// sRGB, user specified gamut
	if( GetOutputDevice() == TONEMAPPER_OUTPUT_sRGB)
	{
		// Convert from sRGB to specified output gamut	
		//float3 OutputGamutColor = mul( AP1_2_Output, mul( sRGB_2_AP1, FilmColor ) );

		// FIXME: Workaround for UE-29935, pushing all colors with a 0 component to black output
		// Default parameters seem to cancel out (sRGB->XYZ->AP1->XYZ->sRGB), so should be okay for a temp fix
		float3 OutputGamutColor = WorkingColorSpace.bIsSRGB ? FilmColor : mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, FilmColor ) );

		// Apply conversion to sRGB (this must be an exact sRGB conversion else darks are bad).
		OutDeviceColor = LinearToSrgb( OutputGamutColor );
	}

	// Rec 709, user specified gamut
	else if( GetOutputDevice() == TONEMAPPER_OUTPUT_Rec709)
	{
		// Convert from the working color space to specified output gamut
		float3 OutputGamutColor = mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, FilmColor ) );

		// Didn't profile yet if the branching version would be faster (different linear segment).
		OutDeviceColor = LinearTo709Branchless( OutputGamutColor );
	}

	// ACES transform with PQ/2084 encoding, user specified gamut 
	else if( GetOutputDevice() == TONEMAPPER_OUTPUT_ACES1000nitST2084 || GetOutputDevice() == TONEMAPPER_OUTPUT_ACES2000nitST2084)
	{
#if USE_ACES_2
		FColorSpace EncodingColorSpace = GetColorSpace(TONEMAPPER_GAMUT_Rec2020_D65);
		// TODO: The display limiting color space should be user-exposed as part of display characterization.
		FColorSpace LimitingColorSpace = GetColorSpace(TONEMAPPER_GAMUT_Rec2020_D65);
		float3 ODTColor = ACESOutputTransform(
			FilmColor * TotalSceneColorMultiplier,
			(float3x3)WorkingColorSpace.ToAP0,
			OutputMaxLuminance,
			ACESReachTable,
			ACESGamutTable,
			ACESGammaTable,
			bScaleWhite,
			LimitingColorSpace.RGBtoXYZ,
			LimitingColorSpace.XYZtoRGB,
			EncodingColorSpace);
#else
		FACESTonemapParams AcesParams = ComputeACESTonemapParams(ACESMinMaxData, ACESMidData, ACESCoefsLow_0, ACESCoefsHigh_0, ACESCoefsLow_4, ACESCoefsHigh_4, ACESSceneColorMultiplier, ACESGamutCompression);
		float3 ODTColor = ACESOutputTransform( FilmColor, (float3x3)WorkingColorSpace.ToAP0, AcesParams);
		ODTColor = mul(AP1_2_Output, ODTColor);
#endif

		// Apply conversion to ST-2084 (Dolby PQ)
		OutDeviceColor = LinearToST2084( ODTColor );
	}

	// ACES transform to linear ScRGB
	else if( GetOutputDevice() == TONEMAPPER_OUTPUT_ACES1000nitScRGB || GetOutputDevice() == TONEMAPPER_OUTPUT_ACES2000nitScRGB)
	{
#if USE_ACES_2
		// TODO: The display limiting color space should be user-exposed as part of display characterization.
		FColorSpace LimitingColorSpace = GetColorSpace(TONEMAPPER_GAMUT_Rec2020_D65);
		float3 ODTColor = ACESOutputTransform(
			FilmColor * TotalSceneColorMultiplier,
			(float3x3)WorkingColorSpace.ToAP0,
			OutputMaxLuminance,
			ACESReachTable,
			ACESGamutTable,
			ACESGammaTable,
			bScaleWhite,
			LimitingColorSpace.RGBtoXYZ,
			LimitingColorSpace.XYZtoRGB,
			GetColorSpace(TONEMAPPER_GAMUT_sRGB_D65));
		OutDeviceColor = ODTColor / UE_SCRGB_WHITE_NITS;
#else
		FACESTonemapParams AcesParams = ComputeACESTonemapParams(ACESMinMaxData, ACESMidData, ACESCoefsLow_0, ACESCoefsHigh_0, ACESCoefsLow_4, ACESCoefsHigh_4, ACESSceneColorMultiplier, ACESGamutCompression);
		float3 ODTColor = ACESOutputTransform(FilmColor, (float3x3) WorkingColorSpace.ToAP0, AcesParams);
		OutDeviceColor = mul(AP1_2_sRGB, ODTColor / UE_SCRGB_WHITE_NITS);
#endif
	}
	
	else if( GetOutputDevice() == TONEMAPPER_OUTPUT_LinearEXR)
	{
		float3 OutputGamutColor = mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, FilmColor ) );
		OutDeviceColor = LinearToST2084( OutputGamutColor );
	}
	// Linear HDR, including all color correction, but no tone curve
	else if( GetOutputDevice() == TONEMAPPER_OUTPUT_NoToneCurve)
	{
		OutDeviceColor = FilmColor;
	}
	// "Linear" including all color correction and the tone curve, but no device gamma
	else if (GetOutputDevice() == TONEMAPPER_OUTPUT_WithToneCurve)
	{
		float3 OutputGamutColor = mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, SceneColor.rgb ) ); 

		OutDeviceColor = OutputGamutColor;
	}

	// OutputDevice == TONEMAPPER_OUTPUT_ExplicitGammaMapping
	// Gamma 2.2, user specified gamut
	else
	{
		// Convert from the working color space to specified output gamut
		float3 OutputGamutColor = mul( AP1_2_Output, mul( (float3x3)WorkingColorSpace.ToAP1, FilmColor ) );

		// This is different than the prior "gamma" curve adjustment (but reusing the variable).
		// For displays set to a gamma colorspace.
		// Note, MacOSX native output is raw gamma 2.2 not sRGB!
		OutDeviceColor = pow( OutputGamutColor, InverseGamma.z );
	}

	OutColor.rgb = OutDeviceColor;
	OutColor.a = 1;

	return OutColor;
}

// pixel shader entry point
void MainPS(
	in noperspective float2 UV : TEXCOORD0,
	float4 SvPosition : SV_POSITION,		// after all interpolators
	out float4 OutColor : SV_Target0
	)
{
    OutColor = DeviceEncodingOnlyCommonPS(UV, SvPosition);
}

#if COMPUTESHADER
RWTexture2D<float4> RWOutputTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float4 SvPosition = float4((float2)DispatchThreadId + Output_ViewportMin + 0.5f, 0.0f, 1.0f);
	float2 UV = SvPosition.xy * Output_ExtentInverse;
	float4 InScreenPos = float4(UV*2-1,0,1);

	if (IsComputeUVOutOfBounds(UV))
	{
		return;
	}

	float4 OutColor = DeviceEncodingOnlyCommonPS(UV, SvPosition);

	uint2 PixelPos = DispatchThreadId + Output_ViewportMin;
	RWOutputTexture[PixelPos] = OutColor;
}
#endif


