// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDownsample.usf: PostProcessing down sample.
=============================================================================*/

#define SCENE_TEXTURES_DISABLED 1

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"

#define DOWNSAMPLE_QUALITY_LOW 0
#define DOWNSAMPLE_QUALITY_HIGH 1

#ifndef DOWNSAMPLE_QUALITY
#error DOWNSAMPLE_QUALITY is not defined.
#endif

Texture2D InputTexture;

SamplerState InputSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

float4 SampleInput(float2 UV)
{
	UV = clamp(UV, Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);

	return Texture2DSampleLevel(InputTexture, InputSampler, UV, 0);
}

float4 DownsampleCommon(float2 UV)
{
	float4 OutColor;

#if DOWNSAMPLE_QUALITY == DOWNSAMPLE_QUALITY_LOW
	// Output: low quality, 1 filtered sample
	OutColor = SampleInput(UV);
#elif DOWNSAMPLE_QUALITY == DOWNSAMPLE_QUALITY_HIGH
	// Output: float4(RGBA), 4 filtered samples
	float2 UVs[4];

	// Blur during downsample (4x4 kernel) to get better quality especially for HDR content.
	UVs[0] = UV + Input_ExtentInverse * float2(-1, -1);
	UVs[1] = UV + Input_ExtentInverse * float2( 1, -1);
	UVs[2] = UV + Input_ExtentInverse * float2(-1,  1);
	UVs[3] = UV + Input_ExtentInverse * float2( 1,  1);

	float4 Sample[4];

	UNROLL
	for(uint i = 0; i < 4; ++i)
	{
		Sample[i] = SampleInput(UVs[i]);
	}

	OutColor = (Sample[0] + Sample[1] + Sample[2] + Sample[3]) * 0.25f;

	// Fixed rarely occurring yellow color tint of the whole viewport (certain viewport size, need to investigate more)
	OutColor.rgb = max(float3(0,0,0), OutColor.rgb);
#else
#error Invalid quality level specified.
#endif

	return OutColor;
}

// pixel shader entry point
#if PIXELSHADER

void MainPS(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	const float2 UV = SvPosition.xy * Output_ExtentInverse;

	OutColor = DownsampleCommon(UV);
}

#elif COMPUTESHADER

FScreenTransform DispatchThreadIdToInputUV;

RWTexture2D<float4> OutComputeTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float2 InputUV = ApplyScreenTransform(float2(DispatchThreadId), DispatchThreadIdToInputUV);
	uint2 OutputPixelPos = DispatchThreadId + Output_ViewportMin;

	if (all(OutputPixelPos < Output_ViewportMax))
	{
		OutComputeTexture[OutputPixelPos] = DownsampleCommon(InputUV);
	}
}

#endif 
