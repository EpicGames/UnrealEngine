// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredShadingCommon.ush"
#include "BRDF.ush"
#include "PositionReconstructionCommon.ush"
#include "Substrate/Substrate.ush"
#include "Substrate/SubstrateEvaluation.ush"

struct FExposureMaterial
{
	float3 TranslatedWorldPosition;
	float3 WorldNormal;
	float3 DirectionalAlbedo;

	bool bIsValid;
	bool bHasBackfaceLighting;
	bool bHasValidDirectionalAlbedo;
};

// same weights as CalculateEyeAdaptationLuminance
// but skip the min since we apply EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance here
float CalculateEyeAdaptationLuminanceWithoutMin(float3 Color)
{
	return dot(Color, EyeAdaptation_LuminanceWeights);
}

FExposureMaterial GetExposureMaterial(uint2 InPixelPos)
{
	FExposureMaterial Out = (FExposureMaterial)0;
#if SHADING_PATH_DEFERRED
	#if SUBSTRATE_GBUFFER_FORMAT==1
	{	
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(InPixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(InPixelPos, 0)));

		const float DeviceZ = ConvertFromDeviceZ(SceneDepthTexture.Load(int3(InPixelPos, 0)).r);
		const float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDeviceZ(InPixelPos, DeviceZ);
		const float3 V = normalize(View.TranslatedWorldCameraOrigin - TranslatedWorldPosition);

		float3 DiffuseDirectionalAlbedo = 0;
		float3 DirectionalAlbedo = 0;
		bool bHasMaterialBackfaceDiffuse = false;
		{
			FSubstrateDeferredLighting Out = GetInitialisedSubstrateDeferredLighting();
			const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);
			Substrate_for(uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
			{
				FSubstrateBSDF BSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
				FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V);
				const float3 BSDFThroughput = LuminanceWeight(SubstrateBSDFContext, BSDF);

				// Evaluate environment lighting
				FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);

				DiffuseDirectionalAlbedo += BSDFThroughput * SubstrateEnvLight.DiffuseWeight;
				DirectionalAlbedo        += BSDFThroughput * SubstrateEnvLight.DiffuseWeight;
				DirectionalAlbedo        += BSDFThroughput * SubstrateEnvLight.SpecularWeight;

				const bool bHasBackfaceDiffuse = SubstrateGetBSDFType(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && BSDF.HasBackScattering();
				if (bHasBackfaceDiffuse)
				{
					DirectionalAlbedo += BSDFThroughput * SubstrateEnvLight.DiffuseBackFaceWeight; // SubstrateEnvLight.DiffuseBackFaceWeight is already divided PI
				}
				if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
				{
					DirectionalAlbedo += BSDFThroughput * SubstrateEnvLight.SpecularHazeWeight;
				}
			}
		}

		Out.TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDeviceZ(InPixelPos, DeviceZ);
		Out.WorldNormal = TopLayerData.WorldNormal;
		Out.DirectionalAlbedo = DirectionalAlbedo;
		Out.bIsValid = SubstratePixelHeader.IsSubstrateMaterial();
		Out.bHasBackfaceLighting = bHasMaterialBackfaceDiffuse;
		Out.bHasValidDirectionalAlbedo = CalculateEyeAdaptationLuminanceWithoutMin(DiffuseDirectionalAlbedo) > EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance;
	}
	#else // SUBSTRATE_GBUFFER_FORMAT==0 or LEGACY
	{
		const float2 GBufferUV = (InPixelPos + 0.5f) * View.BufferSizeAndInvSize.zw;
		const FGBufferData GBufferData = GetGBufferDataFromSceneTextures(GBufferUV);
		const float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDepth(GBufferUV, GBufferData.Depth);

		// Calculate approximate illuminance from SceneColor and GBufferData
		// Illuminance ~= (SceneColorLuminance - Emissive) / (DiffuseColor + SubsurfaceColor + EnvBRDF)
		float3 Denominator = 0.f;
		Denominator += GBufferData.DiffuseColor;
		if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE
			|| GBufferData.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN
			|| GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE
			|| GBufferData.ShadingModelID == SHADINGMODELID_CLOTH
			|| GBufferData.ShadingModelID == SHADINGMODELID_EYE)
		{
			float3 SubsurfaceColor = ExtractSubsurfaceColor(GBufferData);

			if (GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
			{
				SubsurfaceColor *= 1.0f / PI;
			}

			Denominator += SubsurfaceColor;
		}

		{
			const float3 CameraVector = normalize(View.TranslatedWorldCameraOrigin - TranslatedWorldPosition);

			const float3 N = GBufferData.WorldNormal;
			const float3 V = CameraVector;
			const float3 EnvBrdf = EnvBRDF(GBufferData.SpecularColor, GBufferData.Roughness, max(0.0, dot(N, V)));

			Denominator += EnvBrdf;
		}

		Out.TranslatedWorldPosition = TranslatedWorldPosition;
		Out.WorldNormal = GBufferData.WorldNormal;
		Out.DirectionalAlbedo = Denominator;
		Out.bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
		Out.bHasBackfaceLighting = GetShadingModelRequiresBackfaceLighting(GBufferData.ShadingModelID);
		Out.bHasValidDirectionalAlbedo = CalculateEyeAdaptationLuminanceWithoutMin(GBufferData.DiffuseColor) > EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance;
	}
	#endif // SUBSTRATE_GBUFFER_FORMAT
#endif
	return Out;
}