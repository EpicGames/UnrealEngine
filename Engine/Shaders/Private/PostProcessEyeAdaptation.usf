// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessEyeAdaptation.usf: PostProcessing eye adaptation
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "PostProcessHistogramCommon.ush"
#include "PostProcessExposureMaterial.ush"

#if COMPUTESHADER

#define RadianceCacheInterpolation LumenGIVolumeStruct
#include "Lumen/LumenTranslucencyVolumeShared.ush"

#include "DeferredShadingCommon.ush"
#include "BRDF.ush"
#include "PositionReconstructionCommon.ush"
#include "SHCommon.ush"
#include "/Engine/Private/ScreenPass.ush"

RWStructuredBuffer<float4> RWEyeAdaptationBuffer;

#endif

StructuredBuffer<float4> EyeAdaptationBuffer;

Texture2D HistogramTexture;

Texture2D ColorTexture;

SamplerState ColorSampler;

uint2 Color_ViewportMin;
uint2 Color_ViewportMax;

float ComputeWeightedTextureAverageAlpha(
	Texture2D Texture,
	uint2 RectMin,
	uint2 RectMax)
{
	// The inverse of the Region of Interest size.
	const float InvRectWidth = 1.0f / float(RectMax.x - RectMin.x);
	const float InvRectHeight = 1.0f / float(RectMax.y - RectMin.y);

	// use product of linear weight in x and y.
	float Average = 0.0f;
	float WeightTotal = 0.0f;

	for (uint i = RectMin.x; i < RectMax.x; ++i)
	{
		for (uint j = RectMin.y; j < RectMax.y; ++j)
		{
			float2 ScreenUV = float2(float(i)*InvRectWidth,float(j)*InvRectHeight);

			float Weight = max(AdaptationWeightTexture(ScreenUV),0.05f);

			WeightTotal += Weight;

			// Accumulate values from alpha channel.
			float Sample = Texture.Load(int3(i, j, 0)).w;
			Average += Weight * Sample;
		}
	}

	Average /= WeightTotal;
	return Average;
}

float2 ComputeWeightedTextureAverageAlphaSubRegion(
	Texture2D Texture,
	uint2 SubRectMin,
	uint2 SubRectMax,
	uint2 RectMin,
	uint2 RectMax)
{
	if( ((SubRectMax.x - SubRectMin.x)==0) || ((SubRectMax.y - SubRectMin.y)==0) )
	{
		return float2(0.0, 0.0000000001f);
	}
	
	// The inverse of the Region of Interest size.
	const float InvRectWidth = 1.f / float(RectMax.x - RectMin.x);
	const float InvRectHeight = 1.f / float(RectMax.y - RectMin.y);

	// use product of linear weight in x and y.
	float Value = 0.0f;
	float WeightTotal = 0.0f;

	for (uint i = SubRectMin.x; i < SubRectMax.x; ++i)
	{
		// for precision, accumulate in rows
		float RowValue = 0.0;
		float RowWeight = 0.0;
		for (uint j = SubRectMin.y; j < SubRectMax.y; ++j)
		{
			float2 ScreenUV = float2(float(i)*InvRectWidth,float(j)*InvRectHeight);

			float Weight = max(AdaptationWeightTexture(ScreenUV),0.05f);

			// Accumulate values from alpha channel.
			float Sample = Texture.Load(int3(i, j, 0)).w;

			{
				// apply EyeAdaptation_BlackHistogramBucketInfluence using similar logic as Histogram method
				float fBucket = saturate(Sample) * (HISTOGRAM_SIZE - 1);

				// Find two discrete buckets that straddle the continuous histogram position.
				uint Bucket0 = (uint)(fBucket);
				if (Bucket0 == 0)
				{
					Weight *= EyeAdaptation_BlackHistogramBucketInfluence;
				}
			}

			RowValue += Weight * Sample;
			RowWeight += Weight;
		}

		Value += RowValue;
		WeightTotal += RowWeight;
	}

	return float2(Value, max(WeightTotal,0.0000000001f));
}

float2 ComputeTextureAverageAlphaSubRegion(
	Texture2D Texture,
	uint2 SubRectMin,
	uint2 SubRectMax,
	uint2 RectMin,
	uint2 RectMax)
{
	if (((SubRectMax.x - SubRectMin.x) == 0) || ((SubRectMax.y - SubRectMin.y) == 0))
	{
		return float2(0.0, 0.0000000001f);
	}

	// The inverse of the Region of Interest size.
	const float InvRectWidth = 1.f / float(RectMax.x - RectMin.x);
	const float InvRectHeight = 1.f / float(RectMax.y - RectMin.y);

	// use product of linear weight in x and y.
	float Value = 0.0f;

	for (uint i = SubRectMin.x; i < SubRectMax.x; ++i)
	{
		// for precision, accumulate in rows
		float RowValue = 0.0;
		for (uint j = SubRectMin.y; j < SubRectMax.y; ++j)
		{
			float2 ScreenUV = float2(float(i) * InvRectWidth, float(j) * InvRectHeight);

			// Accumulate values from alpha channel.
			float Sample = Texture.Load(int3(i, j, 0)).w;
			RowValue += Sample;
		}

		Value += RowValue;
	}

	const float NumSamples = (SubRectMax.x - SubRectMin.x) * (SubRectMax.y - SubRectMin.y);

	return float2(Value, NumSamples);
}

float4 EyeAdaptationCommon(float AverageSceneLuminance, float OldExposureScale)
{
	const float TargetAverageLuminance = clamp(AverageSceneLuminance, EyeAdaptation_MinAverageLuminance, EyeAdaptation_MaxAverageLuminance);

	// White point luminance is target luminance divided by 0.18 (18% grey).
	const float TargetExposure = TargetAverageLuminance / 0.18;

	const float EyeAdaptationEV100 = log2((AverageSceneLuminance / 0.18) / EyeAdaptation_LuminanceMax);
	const float LUT_U = EyeAdaptationEV100 * EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.x + EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.y;
	const float ExposureCompensationFromCurveLUT = Texture2DSampleLevel(EyeAdaptation_ExposureCompensationCurveLUT, EyeAdaptation_ExposureCompensationCurveSampler, float2(LUT_U, 0.5f), 0).x;

	const float MiddleGreyExposureCompensation = EyeAdaptation_ExposureCompensationSettings * EyeAdaptation_ExposureCompensationCurve * ExposureCompensationFromCurveLUT; // we want the average luminance remapped to 0.18, not 1.0
	const float OldExposure = MiddleGreyExposureCompensation / (OldExposureScale != 0 ? OldExposureScale : 1.0f);

	// eye adaptation changes over time
	const float EstimatedExposure = ComputeEyeAdaptation(OldExposure, TargetExposure, EyeAdaptation_DeltaWorldTime);

	// maybe make this an option to avoid hard clamping when transitioning between different exposure volumes?
	const float SmoothedExposure = clamp(EstimatedExposure, EyeAdaptation_MinAverageLuminance / .18f, EyeAdaptation_MaxAverageLuminance / .18f);

	const float SmoothedExposureScale = 1.0f / max(0.0001f, SmoothedExposure);
	const float TargetExposureScale = 1.0f / max(0.0001f, TargetExposure);

	float4 OutData;
	// Output the number that will rescale the image intensity
	OutData.x = MiddleGreyExposureCompensation * SmoothedExposureScale;
	// Output the target value
	OutData.y = MiddleGreyExposureCompensation * TargetExposureScale;
	OutData.z = AverageSceneLuminance;
	OutData.w = MiddleGreyExposureCompensation;

	return OutData;
}

#if COMPUTE_AVERAGE_LOCAL_EXPOSURE
float ComputeAverageLocalExposure(Texture2D HistogramTexture, float ExposureScale, float MiddleGreyExposureCompensation)
{
	// Average Local Exposure is calculated using log(LocalExposure) 
	// since it results in a more consistent value when applied to Pre-Exposure

	const float MiddleGreyLumValue = log2(0.18 * MiddleGreyExposureCompensation * LocalExposure_MiddleGreyExposureCompensation);

	float2 Sum = 0;

	UNROLL for (uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		float LocalValue = GetHistogramBucket(HistogramTexture, i);

		float LogLuminanceAtBucket = ComputeLogLuminanceFromHistogramPosition(float(i) / (float)(HISTOGRAM_SIZE - 1));

		float BaseLogLum = LogLuminanceAtBucket + log2(ExposureScale);
		float LogLocalExposure = CalculateLogLocalExposure(BaseLogLum, BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, 0.0f);

		Sum += float2(LogLocalExposure, 1) * LocalValue;
	}

	float AverageLocalExposure = Sum.x / max(0.0001f, Sum.y);
	AverageLocalExposure = exp2(AverageLocalExposure);

	return AverageLocalExposure;
}
#endif

#if COMPUTESHADER
[numthreads(1, 1, 1)]
void EyeAdaptationCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const float AverageSceneLuminance = ComputeEyeAdaptationExposure(HistogramTexture);
	const float OldExposureScale = HistogramTexture.Load(int3(0, 1, 0)).x;

	const float4 OutData0 = EyeAdaptationCommon(AverageSceneLuminance, OldExposureScale);
	

#if COMPUTE_AVERAGE_LOCAL_EXPOSURE
	const float AverageLocalExposure = ComputeAverageLocalExposure(HistogramTexture, OutData0.x, OutData0.w);
#else
	const float AverageLocalExposure = 1.0f;
#endif

	RWEyeAdaptationBuffer[0] = OutData0;
	RWEyeAdaptationBuffer[1] = float4(AverageLocalExposure, 0.0f, 0.0f, 0.0f);
}
#endif

void BasicEyeAdaptationSetupPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	OutColor = Texture2DSample(ColorTexture, ColorSampler, UV);

	// Use max to ensure intensity is never zero (so the following log is well behaved)
	const float Intensity = CalculateEyeAdaptationLuminance(OutColor.xyz * View.OneOverPreExposure);
	const float LogIntensity = clamp(log2(Intensity),-10.0f,20.0f);

	// Store log intensity in the alpha channel: scale to 0,1 range.
	OutColor.w = EyeAdaptation_HistogramScale * LogIntensity + EyeAdaptation_HistogramBias; 
}

#if COMPUTESHADER

#define TGSIZE 16

groupshared float2 SubRectValueWeight[TGSIZE*TGSIZE];

float2 Reduce(uint GIndex, float2 Value)
{
	// Store in LDS
	SubRectValueWeight[GIndex] = Value;
	GroupMemoryBarrierWithGroupSync();

	// Merge the ValueWeight from all threads
	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 1];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 2];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 4];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 8];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 16];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 32];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 64];
	GroupMemoryBarrierWithGroupSync();

	SubRectValueWeight[GIndex] = SubRectValueWeight[GIndex] + SubRectValueWeight[GIndex ^ 128];
	GroupMemoryBarrierWithGroupSync();

	return SubRectValueWeight[0];
}

[numthreads(TGSIZE, TGSIZE, 1)]
void BasicEyeAdaptationCS(uint GIndex : SV_GroupIndex, uint2 GTId : SV_GroupThreadID)
{	
	// Compute scaled Log Luminance Average

	// There are TGSIZE*TGSIZE threads. Each thread will calculate the luminance for its own subregions from a TGSIZE*TGSIZE screen grid
	const uint2 SubRectMin = uint2(
		((Color_ViewportMax.x - Color_ViewportMin.x) * GTId.x) / TGSIZE,
		((Color_ViewportMax.y - Color_ViewportMin.y) * GTId.y) / TGSIZE);

	const uint2 SubRectMax = uint2(
		((Color_ViewportMax.x - Color_ViewportMin.x) * (GTId.x + 1)) / TGSIZE,
		((Color_ViewportMax.y - Color_ViewportMin.y) * (GTId.y + 1)) / TGSIZE);

	const float2 LogLumAveWithWeight = ComputeWeightedTextureAverageAlphaSubRegion(ColorTexture, SubRectMin, SubRectMax, Color_ViewportMin, Color_ViewportMax);

	const float2 ReducedLogLumAveWithWeight = Reduce(GIndex, LogLumAveWithWeight);

	float LogLumAve = ReducedLogLumAveWithWeight.x / ReducedLogLumAveWithWeight.y;

	// Correct for [0,1] scaling
	LogLumAve = (LogLumAve - EyeAdaptation_HistogramBias) / EyeAdaptation_HistogramScale;
	
	// Convert LogLuminanceAverage to Average Intensity
	const float AverageSceneLuminance = exp2(LogLumAve);
	
	// The Exposure Scale (and thus intensity) used in the previous frame
	const float OldExposureScale = EyeAdaptationBuffer[0].x;

	const float4 OutData0 = EyeAdaptationCommon(AverageSceneLuminance, OldExposureScale);

#if COMPUTE_AVERAGE_LOCAL_EXPOSURE
	const float MiddleGreyLumValue = log2(0.18 * OutData0.w * LocalExposure_MiddleGreyExposureCompensation);

	const float2 LogLumSumWithCount = ComputeTextureAverageAlphaSubRegion(ColorTexture, SubRectMin, SubRectMax, Color_ViewportMin, Color_ViewportMax);

	float LogLum = LogLumSumWithCount.x / LogLumSumWithCount.y;
	LogLum = (LogLum - EyeAdaptation_HistogramBias) / EyeAdaptation_HistogramScale;

	const float BaseLogLum = LogLum + log2(OutData0.x);
	const float LogLocalExposure = CalculateLogLocalExposure(BaseLogLum, BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, 0.0f);

	const float2 LogLocalExposureSumWithCount = float2(LogLocalExposure, 1.0f) * LogLumSumWithCount.y;

	const float2 ReducedLogLocalExposureSumWithCount = Reduce(GIndex, LogLocalExposureSumWithCount);

	const float AverageLocalExposure = exp2(ReducedLogLocalExposureSumWithCount.x / ReducedLogLocalExposureSumWithCount.y);
#else
	const float AverageLocalExposure = 1.0f;
#endif
	
#if XBOXONE_PROFILE
	OutData0 = !all(IsFinite(OutData0)) ? float4(1, 1, 1, 0) : OutData0;
#endif
	
	if(GIndex == 0)
	{
		RWEyeAdaptationBuffer[0] = OutData0;
		RWEyeAdaptationBuffer[1] = float4(AverageLocalExposure, 0.0f, 0.0f, 0.0f);
	}
}

#endif

#if COMPUTESHADER

RWTexture2D<float> RWIlluminanceTexture;
SCREEN_PASS_TEXTURE_VIEWPORT(Illuminance)
FScreenTransform IlluminanceRectToColorRect;

[numthreads(8, 8, 1)]
void SetupExposureIlluminance(uint2 ThreadId : SV_DispatchThreadID)
{
	const uint2 OutputTexelPos = ThreadId + Illuminance_ViewportMin;
	const uint2 TexelPos = ApplyScreenTransform(float2(OutputTexelPos + 0.5f), IlluminanceRectToColorRect);

	const float3 Emissive = ColorTexture.Load(int3(TexelPos, 0)).rgb;

	RWIlluminanceTexture[OutputTexelPos] = CalculateEyeAdaptationLuminanceWithoutMin(Emissive);
}

Texture3D TranslucencyLightingVolumeAmbientInner;
Texture3D TranslucencyLightingVolumeAmbientOuter;
Texture3D TranslucencyLightingVolumeDirectionalInner;
Texture3D TranslucencyLightingVolumeDirectionalOuter;
Texture3D TranslucencySpecularDirectionInner;
Texture3D TranslucencySpecularDirectionOuter;

#include "TranslucencyVolumeCommon.ush"

[numthreads(8, 8, 1)]
void CalculateExposureIlluminance(uint2 ThreadId : SV_DispatchThreadID)
{
	const uint2 OutputTexelPos = ThreadId + Illuminance_ViewportMin;
	const uint2 TexelPos = ApplyScreenTransform(float2(OutputTexelPos + 0.5f), IlluminanceRectToColorRect);

	const float3 SceneColor = ColorTexture.Load(int3(TexelPos, 0)).rgb * View.OneOverPreExposure;

	float Luminance = CalculateEyeAdaptationLuminance(SceneColor);

	const FExposureMaterial Material = GetExposureMaterial(TexelPos);

	if (Material.bIsValid)
	{
		bool bUseTranslucencyVolume = !EyeAdaptation_IgnoreMaterialsReconstructFromSceneColor;

		// when DirectionalAlbedo is black there's not enough information in SceneColor to reconstruct illuminance
		if (!Material.bHasValidDirectionalAlbedo)
		{
			bUseTranslucencyVolume = true;
		}

		float Illuminance;

		if (!bUseTranslucencyVolume)
		{
			const float DenominatorLuminance = CalculateEyeAdaptationLuminanceWithoutMin(Material.DirectionalAlbedo);
			// Emissive is copied to IlluminanceTexture after base pass and before lighting pass
			const float Emissive = RWIlluminanceTexture[OutputTexelPos] * View.OneOverPreExposure;

			Illuminance = max(0.0f, Luminance - Emissive) / max(DenominatorLuminance, EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance);
		}
		else
		{
			const float3 TranslucencyEvaluationTranslatedPosition = Material.TranslatedWorldPosition + Material.WorldNormal * EyeAdaptation_IgnoreMaterialsEvaluationPositionBias;

			const float3 LightingPositionOffset = 0;

			float3 InnerVolumeUVs;
			float3 OuterVolumeUVs;
			float FinalLerpFactor;
			ComputeVolumeUVs(TranslucencyEvaluationTranslatedPosition, LightingPositionOffset, InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

			float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
			float3 DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
			
			float3 VolumeLighting = GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, Material.WorldNormal, 1.0f).rgb;
			
			if (Material.bHasBackfaceLighting)
			{
				VolumeLighting += GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, -Material.WorldNormal, 1.0f).rgb;
			}

		#if PROJECT_SUPPORTS_LUMEN && !FORWARD_SHADING
			if (IsLumenTranslucencyGIEnabled())
			{
				// Lumen Dynamic GI + shadowed Skylight
				FDFVector3 TranslucencyEvaluationPosition = DFFastSubtract(TranslucencyEvaluationTranslatedPosition, PrimaryView.PreViewTranslation);

				FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(TranslucencyEvaluationPosition, PrimaryView.WorldToClip, true);

				// Diffuse convolution
				FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(Material.WorldNormal, 1);
				VolumeLighting.rgb += max(half3(0, 0, 0), DotSH(TranslucencyGISH, DiffuseTransferSH)) / PI;

				if (Material.bHasBackfaceLighting)
				{
					FTwoBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH(-Material.WorldNormal, 1);
					VolumeLighting.rgb += max(half3(0, 0, 0), DotSH(TranslucencyGISH, SubsurfaceTransferSH)) / PI;
				}
			}
		#endif
			else
			{
				// TODO: Use SkyLight
			}

			Illuminance = CalculateEyeAdaptationLuminance(VolumeLighting.rgb);
		}

		Luminance = Illuminance * lerp(EyeAdaptation_IgnoreMaterialsLuminanceScale, 1.0f, EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance);
	}
	
	RWIlluminanceTexture[OutputTexelPos] = Luminance * View.PreExposure;
}

#endif // COMPUTESHADER
