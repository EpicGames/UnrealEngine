// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

StructuredBuffer<float4> EyeAdaptationBuffer;

#if COMPUTESHADER
RWTexture2D<float4> RWEyeAdaptationTexture;

[numthreads(1, 1, 1)]
void CopyEyeAdaptationToTextureCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	RWEyeAdaptationTexture[DispatchThreadId] = EyeAdaptationBuffer[0];
}

#endif // COMPUTESHADER
