// Copyright Epic Games, Inc. All Rights Reserved.

#if SHADER_GBUFFER_PICKING
#define ALLOW_SSS_MATERIAL_OVERRIDE 0
#endif

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "ScreenPass.ush"

#if SUBSTRATE_ENABLED
#include "ToonProfileCommon.ush"
#include "Substrate/Substrate.ush"
#endif

#define SUPPORT_GBUFFER_FETCH (SHADING_PATH_DEFERRED && FEATURE_LEVEL >= FEATURE_LEVEL_SM5)

float2 Input_ExtentInverse;

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D OriginalSceneColorTexture;
SamplerState OriginalSceneColorSampler;

#if SUPPORT_GBUFFER_FETCH

// 0: good, 1: heavy performance cost, 2:warning (very uncommon material), 3:error (impossible material attributes)
uint ComputeAdvice(FGBufferData BufferData, float3 OriginalSceneColor)
{
	float SubsurfaceLum = 0;
	const float OneThird = 1.0f / 3.0f;

	FLATTEN if(BufferData.ShadingModelID != SHADINGMODELID_UNLIT)
	{
		FLATTEN if(BufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE)
		{
			SubsurfaceLum = dot(OneThird.xxx, ExtractSubsurfaceColor(BufferData));
		}

		float DiffuseLum = dot(OneThird.xxx, BufferData.DiffuseColor);
		float SpecularLum = dot(OneThird.xxx, BufferData.SpecularColor);

		float TotalLum = DiffuseLum + SpecularLum + SubsurfaceLum;

		float OriginalSceneColorLum = dot(OneThird.xxx, OriginalSceneColor);

		FLATTEN if(TotalLum < 0.01f && OriginalSceneColorLum < 0.001f)
		{
			// The material is very dark. 
			// Such a material would be very unlikely (Maybe it can only be created in a lab).
			return 2;
		}

		FLATTEN if(TotalLum > 1.2f)
		{
			// The diffuse and specular cause the material to emit more light than it receives.
			return 3;
		}
	}

	return 0;
}

uint ComputeAdvice(float2 UV)
{
	// Ideally without GI as we want the emissive, we should do that later.
	float3 OriginalSceneColor = Texture2DSample(OriginalSceneColorTexture, OriginalSceneColorSampler, UV).rgb;
	FGBufferData BufferData = GetGBufferData(UV);

	return ComputeAdvice(BufferData, OriginalSceneColor);
}

// to highlight areas that have unrealistic materials
void HighlightAdvice(inout float3 OutColor, float2 UV, uint2 PixelPos)
{
	uint AdviceInner = ComputeAdvice(UV);
	uint AdviceOuter = 0;

	bool SpecialDotInArea = ((PixelPos.x + PixelPos.y) % 6) == 0 && ((PixelPos.x - PixelPos.y) % 6) == 0;

	AdviceOuter = max(AdviceOuter, ComputeAdvice(UV + float2( 1,  0) * Input_ExtentInverse));
	AdviceOuter = max(AdviceOuter, ComputeAdvice(UV + float2( 0,  1) * Input_ExtentInverse));
	AdviceOuter = max(AdviceOuter, ComputeAdvice(UV + float2(-1,  0) * Input_ExtentInverse));
	AdviceOuter = max(AdviceOuter, ComputeAdvice(UV + float2( 0, -1) * Input_ExtentInverse));

	uint Advice = (AdviceInner == AdviceOuter && !SpecialDotInArea) ? 0 : AdviceOuter;

	FLATTEN if(Advice)
	{
		 FLATTEN if(Advice == 1)
		 {
			 // heavy shading cost
			 OutColor = float3(0, 0, 0.8f);
		 }
		 else
		 FLATTEN if(Advice == 2)
		 {
			 // warning
			 OutColor = float3(0.8f, 0.8f, 0);
		 }
		 else // if(Advice == 3)
		 {
			 // error
			 OutColor = float3(1, 0, 0);
		 }
	}
}
#endif

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	int2 PixelPos = (uint2)SvPosition.xy;
	
	// background is the scene color
	OutColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UV);

#if SUPPORT_GBUFFER_FETCH
	HighlightAdvice(OutColor.rgb, UV, PixelPos);
#endif
}


////////////////////////////////////////////////////////////////////////////////////////////////
// Debug
#if SHADER_GBUFFER_PICKING

#include "ShaderPrint.ush"
#include "SceneTextureParameters.ush"
#include "DeferredShadingCommon.ush"
#include "ShadingModels.ush"
#include "SubsurfaceProfileCommon.ush"
#include "PositionReconstructionCommon.ush"

void PrintShadingMode(inout FShaderPrintContext Ctx, uint In)
{
	if (In == SHADINGMODELID_UNLIT) 				{ Print(Ctx, TEXT("UNLIT"), FontRed); return; }
	if (In == SHADINGMODELID_DEFAULT_LIT) 			{ Print(Ctx, TEXT("DEFAULT_LIT"), FontRed); return; }
	if (In == SHADINGMODELID_SUBSURFACE) 			{ Print(Ctx, TEXT("SUBSURFACE"), FontRed); return; }
	if (In == SHADINGMODELID_PREINTEGRATED_SKIN) 	{ Print(Ctx, TEXT("PREINTEGRATED_SKIN"), FontRed); return; }
	if (In == SHADINGMODELID_CLEAR_COAT) 			{ Print(Ctx, TEXT("CLEAR_COAT"), FontRed); return; }
	if (In == SHADINGMODELID_SUBSURFACE_PROFILE)	{ Print(Ctx, TEXT("SUBSURFACE_PROFILE"), FontRed); return; }
	if (In == SHADINGMODELID_TWOSIDED_FOLIAGE) 		{ Print(Ctx, TEXT("TWOSIDED_FOLIAGE"), FontRed); return; }
	if (In == SHADINGMODELID_HAIR) 					{ Print(Ctx, TEXT("HAIR"), FontRed); return; }
	if (In == SHADINGMODELID_CLOTH) 				{ Print(Ctx, TEXT("CLOTH"), FontRed); return; }
	if (In == SHADINGMODELID_EYE) 					{ Print(Ctx, TEXT("EYE"), FontRed); return; }
	if (In == SHADINGMODELID_SINGLELAYERWATER)		{ Print(Ctx, TEXT("SINGLELAYERWATER"), FontRed); return; }
	if (In == SHADINGMODELID_THIN_TRANSLUCENT)		{ Print(Ctx, TEXT("THIN_TRANSLUCENT"), FontRed); return; }
	if (In == SHADINGMODELID_SUBSTRATE)				{ Print(Ctx, TEXT("SUBSTRATE"), FontRed); return; }
	if (In == SHADINGMODELID_SUBSTRATE_TOON)		{ Print(Ctx, TEXT("TOON"), FontRed); return; }
	if (In == SHADINGMODELID_NUM)					{ Print(Ctx, TEXT("NUM"), FontRed); return; }
	if (In == SHADINGMODELID_MASK) 					{ Print(Ctx, TEXT("MASK"), FontRed); return; }
}

void Newline(inout FShaderPrintContext Ctx, inout float2 MaxRect)
{
	MaxRect.x = max(MaxRect.x, Ctx.Pos.x);
	MaxRect.y = Ctx.Pos.y;
	Newline(Ctx);
}

void PrintBackgroundRect(inout FShaderPrintContext Ctx, float2 StartRect, float2 EndRect)
{
	const float4 RectBackgroundColor = float4(1, 1, 1, 0.25f);
	AddFilledQuadSS(
		(StartRect - ShaderPrintData.FontSize) * ShaderPrintData.Resolution,
		(EndRect + ShaderPrintData.FontSize) * ShaderPrintData.Resolution,
		RectBackgroundColor);
}

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
#if (!SUBSTRATE_ENABLED || SUBSTRATE_GBUFFER_FORMAT==0) && SUPPORT_GBUFFER_FETCH
	// Draw stats
	if (all(DispatchThreadId == 0))
	{
		FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(50, 50));
		const FScreenSpaceData ScreenSpaceData = GetScreenSpaceDataUint(ShaderPrintData.CursorCoord * View.ViewResolutionFraction, true);
		const FGBufferData Data = ScreenSpaceData.GBuffer;

		float2 RectMin = Ctx.Pos;
		float2 RectMax = Ctx.StartPos;

		RectMin = Ctx.Pos;
		Print(Ctx, TEXT("Global"), FontOrange);Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Scene Depth       : "), FontWhite); Print(Ctx, Data.Depth, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Diffuse Color     : "), FontWhite); Print(Ctx, Data.DiffuseColor, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Specular Color    : "), FontWhite); Print(Ctx, Data.SpecularColor, FontYellow); Newline(Ctx, RectMax);
		PrintBackgroundRect(Ctx, RectMin, RectMax);
		Newline(Ctx);

		RectMin = Ctx.Pos;
		Print(Ctx, TEXT("Tangent basis"), FontOrange);Newline(Ctx, RectMax);
		Print(Ctx, TEXT("World Normal      : "), FontWhite); Print(Ctx, Data.WorldNormal, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("World Tangent     : "), FontWhite); Print(Ctx, Data.WorldTangent, FontYellow); Newline(Ctx, RectMax);
		PrintBackgroundRect(Ctx, RectMin, RectMax);
		Newline(Ctx);

		RectMin = Ctx.Pos;
		Print(Ctx, TEXT("Common"), FontOrange);Newline(Ctx, RectMax);
		Print(Ctx, TEXT("BaseColor         : "), FontWhite); Print(Ctx, Data.BaseColor, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Metallic          : "), FontWhite); Print(Ctx, Data.Metallic, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Specular          : "), FontWhite); Print(Ctx, Data.Specular, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Roughness         : "), FontWhite); Print(Ctx, Data.Roughness, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Anisotropy        : "), FontWhite); Print(Ctx, Data.Anisotropy, FontYellow); Newline(Ctx, RectMax);
		PrintBackgroundRect(Ctx, RectMin, RectMax);
		Newline(Ctx);

		RectMin = Ctx.Pos;
		Print(Ctx, TEXT("Occlusion / Ind. Lighting"), FontOrange);Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Generic AO        : "), FontWhite); Print(Ctx, Data.GenericAO, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("GBuffer AO        : "), FontWhite); Print(Ctx, Data.GBufferAO, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Ind. Sample Occ.  : "), FontWhite); Print(Ctx, Data.DiffuseIndirectSampleOcclusion, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Ind. Irradiance   : "), FontWhite); Print(Ctx, Data.IndirectIrradiance, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Prec.Shadow Factor: "), FontWhite); Print(Ctx, Data.PrecomputedShadowFactors, FontYellow); Newline(Ctx, RectMax);
		Newline(Ctx);
		Print(Ctx, TEXT("CastContactShadow : "), FontWhite); PrintBool(Ctx, CastContactShadow(Data)); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("HasDynIndShadCast : "), FontWhite); PrintBool(Ctx, HasDynamicIndirectShadowCasterRepresentation(Data)); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("IsFirstPerson     : "), FontWhite); PrintBool(Ctx, IsFirstPerson(Data)); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("HasPrecShadowMask : "), FontWhite); PrintBool(Ctx, HasPrecShadowMask(Data)); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("HasZeroPrecShadow : "), FontWhite); PrintBool(Ctx, HasZeroPrecShadowMask(Data)); Newline(Ctx, RectMax);
		PrintBackgroundRect(Ctx, RectMin, RectMax);
		Newline(Ctx);

		RectMin = Ctx.Pos;
		Print(Ctx, TEXT("Custom"), FontOrange);Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Shading Model ID  : "), FontWhite); PrintShadingMode(Ctx, Data.ShadingModelID); Newline(Ctx, RectMax);
		Print(Ctx, TEXT(" Raw Custom Data  : "), FontWhite); Print(Ctx, Data.CustomData, FontYellow); Newline(Ctx, RectMax);
		if (Data.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
		{
		Print(Ctx, TEXT("Clear Coat        : "), FontWhite); Print(Ctx, Data.CustomData.x, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Clear Coat Roughn.: "), FontWhite); Print(Ctx, Data.CustomData.y, FontYellow); Newline(Ctx, RectMax);
		}
		else if (Data.ShadingModelID == SHADINGMODELID_CLOTH)
		{
		Print(Ctx, TEXT("Fuzz Color        : "), FontWhite); Print(Ctx, Data.CustomData.xyz, FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Fuzz Amount       : "), FontWhite); Print(Ctx, Data.CustomData.a, FontYellow); Newline(Ctx, RectMax);
		}
		else if (Data.ShadingModelID == SHADINGMODELID_SUBSURFACE || Data.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN || Data.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
		{
		Print(Ctx, TEXT("Subsurface Color  : "), FontWhite); Print(Ctx, ExtractSubsurfaceColor(Data), FontYellow); Newline(Ctx, RectMax);
		Print(Ctx, TEXT("Opacity           : "), FontWhite); Print(Ctx, Data.CustomData.a, FontYellow); Newline(Ctx, RectMax);
		}
		else if (Data.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE)
		{
			const uint SSSProfileID = ExtractSubsurfaceProfileInt(Data.CustomData.x);
			const half Opacity = Data.CustomData.a;
			float LobeMix = 0;
			float Lobe0Roughness = 0;
			float Lobe1Roughness = 0;
			GetProfileDualSpecular(SSSProfileID, Data.Roughness, Opacity, Lobe0Roughness, Lobe1Roughness, LobeMix);

			Print(Ctx, TEXT("SSS Profil ID     : "), FontWhite); Print(Ctx, SSSProfileID); Newline(Ctx, RectMax);
			Print(Ctx, TEXT("SSS Lobe Mix      : "), FontWhite); Print(Ctx, LobeMix); Newline(Ctx, RectMax);
			Print(Ctx, TEXT("SSS Roughness0    : "), FontWhite); Print(Ctx, Lobe0Roughness); Newline(Ctx, RectMax);
			Print(Ctx, TEXT("SSS Roughness1    : "), FontWhite); Print(Ctx, Lobe1Roughness); Newline(Ctx, RectMax);
			Print(Ctx, TEXT("SSS SpecColor     : "), FontWhite); Print(Ctx, Data.SpecularColor); Newline(Ctx, RectMax);
			
		}
	#if SUBSTRATE_ENABLED
		else if (Data.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
		{
			float2 PatternUVs = 0.0f;
			float ToonProfileId = 0.0f;
			UnpackToonCustomData(PackRGBA8(Data.CustomData), PatternUVs, ToonProfileId);

			Print(Ctx, TEXT("Pattern UVs        : "), FontWhite); Print(Ctx, PatternUVs); Newline(Ctx, RectMax);
			Print(Ctx, TEXT("Toon Profil ID     : "), FontWhite); Print(Ctx, ToonProfileId); Newline(Ctx, RectMax);
		}
	#endif
		PrintBackgroundRect(Ctx, RectMin, RectMax);
		Newline(Ctx);
		
		// Draw tangent vector (strands)
		{
			const float2 UV = (ShaderPrintData.CursorCoord + 0.5f) * View.BufferSizeAndInvSize.zw;
			const float3 WorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, Data.Depth);
			AddLineTWS(WorldPosition, WorldPosition + Data.WorldNormal, ColorRed, ColorYellow);
		}

		if (CLEAR_COAT_BOTTOM_NORMAL && Data.Anisotropy != 0)
		{
			const float2 UV = (ShaderPrintData.CursorCoord + 0.5f) * View.BufferSizeAndInvSize.zw;
			const float3 WorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, Data.Depth);
			AddLineTWS(WorldPosition, WorldPosition + Data.WorldNormal, ColorRed, ColorYellow);
		}
	}
#endif
}

#endif // SHADER_GBUFFER_PICKING