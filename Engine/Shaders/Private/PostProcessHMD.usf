/************************************************************************************

Copyright   :   Copyright 2014 Oculus VR, Inc. All Rights reserved.

Licensed under the Oculus VR SDK License Version 2.0 (the "License");
you may not use the Oculus VR SDK except in compliance with the License,
which is provided at the time of installation or download, or which
otherwise accompanies this software in either electronic or hard copy form.

You may obtain a copy of the License at

http://www.oculusvr.com/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, the Oculus VR SDK
distributed under the License is distributed on an "AS IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.

*************************************************************************************/
/*=============================================================================
	PostProcessHMD.usf: PostProcessing shader to distort and chromaab correction 
	for HMD devices
=============================================================================*/

#include "Common.ush"

Texture2D InputTexture;
SamplerState InputSampler;

float2 EyeToSrcUVScale;
float2 EyeToSrcUVOffset;

void MainVS(
	in float2 Position : ATTRIBUTE0,
	in float2 TexCoord0 : ATTRIBUTE1,
	in float2 TexCoord1 : ATTRIBUTE2, 
	in float2 TexCoord2 : ATTRIBUTE3,
	in float VignetteFactor : ATTRIBUTE4,
	in float TimewarpFactor : ATTRIBUTE5,
	out float4 OutPosition : SV_Position,
	out float4 OutVignetteColor : COLOR0, 
	out float4 OutTexCoord0 : TEXCOORD0,
	out float4 OutTexCoord1 : TEXCOORD1,
	out float4 OutTexCoord2 : TEXCOORD2)
{
	OutPosition.xy = Position.xy;
	OutPosition.z = 0.5;
	OutPosition.w = 1.0;

    // Vertex inputs are in TanEyeAngle space for the R,G,B channels (i.e. after chromatic aberration and distortion).
    // Scale them into the correct [0-1],[0-1] UV lookup space (depending on eye)
	float2 ScaledTexCoord0 = TexCoord0 * EyeToSrcUVScale + EyeToSrcUVOffset;
	float2 ScaledTexCoord1 = TexCoord1 * EyeToSrcUVScale + EyeToSrcUVOffset;
	float2 ScaledTexCoord2 = TexCoord2 * EyeToSrcUVScale + EyeToSrcUVOffset;
	OutTexCoord0 = float4(ScaledTexCoord0.x, ScaledTexCoord0.y, TexCoord0.x, TexCoord0.y);
	OutTexCoord1 = float4(ScaledTexCoord1.x, ScaledTexCoord1.y, TexCoord1.x, TexCoord1.y);
	OutTexCoord2 = float4(ScaledTexCoord2.x, ScaledTexCoord2.y, TexCoord2.x, TexCoord2.y);
	OutVignetteColor = VignetteFactor; // Used for vignette fade.
}

void MainPS(
	in float4 Position : SV_Position,
	in float4 Color : COLOR0,
	in float4 TexCoord0 : TEXCOORD0,
	in float4 TexCoord1 : TEXCOORD1,
	in float4 TexCoord2 : TEXCOORD2,
	out float4 OutColor : SV_Target0)
{
	// Only testing 1 channel for optimization
#if PIXELSHADER // Fix gles error since VS and PS are in the same file
#if METAL_ES3_1_PROFILE || METAL_SM5_IOS_TVOS_PROFILE || METAL_SM5_PROFILE || METAL_SM6_PROFILE // Fix Mac issue with bvec2
	const float2 OOBCoords = TexCoord1.zw - saturate(TexCoord1.zw);
	if( dot(OOBCoords, OOBCoords) != 0.0 )
#else
	if( any(TexCoord1.zw - saturate(TexCoord1.zw)) )
#endif
	{
		discard;
	}
#endif
	float ResultR = Texture2DSample(InputTexture, InputSampler, TexCoord0.xy).r;
	float ResultG = Texture2DSample(InputTexture, InputSampler, TexCoord1.xy).g;
	float ResultB = Texture2DSample(InputTexture, InputSampler, TexCoord2.xy).b;
	OutColor = float4(ResultR * Color.r, ResultG * Color.g, ResultB * Color.b, 1.0);
}
