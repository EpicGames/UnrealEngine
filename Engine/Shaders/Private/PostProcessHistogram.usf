// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessHistogram.usf: PostProcessing histogram
=============================================================================*/

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessHistogramCommon.ush"

#if LOOP_SIZE_X % 2 || LOOP_SIZE_Y % 2
#	 error "Must be even to use gather"
#endif

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

Texture2D InputTexture;
SamplerState InputSampler;

uint PackFloatToUINT32(float v)
{
	return (uint)(v * float(1 << 19));
}

float UnpackUINT32ToFloat(uint v)
{
	float fv = (float)(v);
	return fv * (1.0f / float(1 << 19));
}

// Want to pack 2 floats in a uint (16-bits for each)
// each float is [0,1]
// Each thread samples 8*8 tile = 64 texels
// log2(64) = 6 => 7 bits
// shift float by 16-7 = 9 bits
#define PACK_SCALE float(1 << 9)

uint PackTwoFloatToUINT32(float a, float b)
{
	uint ua = (uint)(a * PACK_SCALE) & 0xFFFF;
	uint ub = (uint)(b * PACK_SCALE) & 0xFFFF;
	return (ua << 16) | ub;
}

void UnpackUINT32ToTwoFloat(uint v, out float a, out float b)
{
	uint ua = v >> 16;
	uint ub = v & 0xFFFF;

	const float h = (1.0f / PACK_SCALE);

	a = ua * h;
	b = ub * h;
}

#if BILATERAL_GRID
// Output bilateral grid texture (UAV)
RWTexture3D<float2> BilateralGridRWTexture;
#else
// Output histogram texture (UAV)
RWTexture2D<float4> HistogramRWTexture;
#endif

RWTexture2D<float> DebugOutput;

// Number of thread groups in the dispatch
uint2 ThreadGroupCount;

// THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms of the size HISTOGRAM_SIZE
groupshared uint SharedHistogram[HISTOGRAM_SIZE][THREADGROUP_SIZEX][THREADGROUP_SIZEY];

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex: SV_GroupIndex)
{
	// todo: can be cleared more efficiently
	// clear all THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms
	UNROLL for (uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		SharedHistogram[i][GroupThreadId.x][GroupThreadId.y] = 0;
	}
	
	GroupMemoryBarrierWithGroupSync();

	// Each thread in the group processes LoopX * LoopY texels of the input.
	const uint2 TileSize = uint2(THREADGROUP_SIZEX * LOOP_SIZEX, THREADGROUP_SIZEY * LOOP_SIZEY);

	// Top left input texel for this group.
	const uint2 LeftTop = Input_ViewportMin + GroupId.xy * TileSize;

	const float2 InvViewSize = Input_ViewportSizeInverse.xy;
	const float2 InvExtent = Input_ExtentInverse.xy;

	const uint2 NumPixelsPerIter = uint2(2, 2);

	// Accumulate all pixels into THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms
	for (uint y = 0; y < THREADGROUP_SIZEY * LOOP_SIZEY; y += THREADGROUP_SIZEY * NumPixelsPerIter.y)
	{
		for (uint x = 0; x < THREADGROUP_SIZEX * LOOP_SIZEX; x += THREADGROUP_SIZEX * NumPixelsPerIter.x)
		{
			uint2 TexelPos = LeftTop + uint2(x, y) + GroupThreadId.xy * NumPixelsPerIter;

			// don't include last column if viewport max is odd
			if(all(TexelPos < Input_ViewportMax - 1))
			{
				// sample at intersection of 4 texels
				float2 UV = (TexelPos + 1.0f) * InvExtent;

				float4 SceneColorR = InputTexture.GatherRed(InputSampler, UV);
				float4 SceneColorG = InputTexture.GatherGreen(InputSampler, UV);
				float4 SceneColorB = InputTexture.GatherBlue(InputSampler, UV);

				SceneColorR *= View.OneOverPreExposure;
				SceneColorG *= View.OneOverPreExposure;
				SceneColorB *= View.OneOverPreExposure;

				float4 LuminanceVal;
				LuminanceVal.x = CalculateEyeAdaptationLuminance(float3(SceneColorR.x, SceneColorG.x, SceneColorB.x));
				LuminanceVal.y = CalculateEyeAdaptationLuminance(float3(SceneColorR.y, SceneColorG.y, SceneColorB.y));
				LuminanceVal.z = CalculateEyeAdaptationLuminance(float3(SceneColorR.z, SceneColorG.z, SceneColorB.z));
				LuminanceVal.w = CalculateEyeAdaptationLuminance(float3(SceneColorR.w, SceneColorG.w, SceneColorB.w));

#if USE_PRECALCULATED_LUMINANCE
				LuminanceVal = SceneColorR;
#elif USE_APPROX_ILLUMINANCE
				// TODO
#endif

#if USE_DEBUG_OUTPUT
				DebugOutput[TexelPos - Input_ViewportMin + uint2(0, 1)] = LuminanceVal.r;
				DebugOutput[TexelPos - Input_ViewportMin + uint2(1, 1)] = LuminanceVal.g;
				DebugOutput[TexelPos - Input_ViewportMin + uint2(1, 0)] = LuminanceVal.b;
				DebugOutput[TexelPos - Input_ViewportMin + uint2(0, 0)] = LuminanceVal.a;
#endif

				// only sample screen weight once per quad
				float2 ScreenUV = (TexelPos.xy - Input_ViewportMin) * InvViewSize;
				float ScreenWeight = AdaptationWeightTexture(ScreenUV);

				UNROLL_N(4)
				for (uint i = 0; i < 4; ++i)
				{
					float LogLuminance = log2(LuminanceVal[i]);
					float LogLuminanceHist = ComputeHistogramPositionFromLogLuminance(LogLuminance);

					// Map the normalized histogram position into texels.
					float fBucket = saturate(LogLuminanceHist) * (HISTOGRAM_SIZE - 1);

					// Find two discrete buckets that straddle the continuous histogram position.
					uint Bucket0 = (uint)(fBucket);
					uint Bucket1 = Bucket0 + 1;

					Bucket0 = min(Bucket0, uint(HISTOGRAM_SIZE - 1));
					Bucket1 = min(Bucket1, uint(HISTOGRAM_SIZE - 1));

					// Weighted blend between the two buckets.
					float Weight1 = frac(fBucket);
					float Weight0 = 1.0f - Weight1;

					// Accumulate the weight to the nearby history buckets.

#if BILATERAL_GRID
					uint LogLuminanceHist0 = PackTwoFloatToUINT32(Weight0, LogLuminanceHist * Weight0);
					uint LogLuminanceHist1 = PackTwoFloatToUINT32(Weight1, LogLuminanceHist * Weight1);

					InterlockedAdd(SharedHistogram[Bucket0][GroupThreadId.x][GroupThreadId.y], LogLuminanceHist0);
					InterlockedAdd(SharedHistogram[Bucket1][GroupThreadId.x][GroupThreadId.y], LogLuminanceHist1);
#else
					// When EyeAdaptation_BlackHistogramBucketInfluence=.0, we will ignore the last bucket. The main use
					// case is so the black background pixels in the editor have no effect. But if we have cases where
					// pixel values can actually be black, we want to set EyeAdaptation_LastHistogramBucketInfluence=1.0.
					// This value is controlled by the cvar "r.EyeAdaptation.BlackHistogramBucketInfluence"
					if (Bucket0 == 0)
					{
						Weight0 *= EyeAdaptation_BlackHistogramBucketInfluence;
					}

					Weight0 *= ScreenWeight;
					Weight1 *= ScreenWeight;

					uint Weight0Int = PackFloatToUINT32(Weight0);
					uint Weight1Int = PackFloatToUINT32(Weight1);

					InterlockedAdd(SharedHistogram[Bucket0][GroupThreadId.x][GroupThreadId.y], Weight0Int);
					InterlockedAdd(SharedHistogram[Bucket1][GroupThreadId.x][GroupThreadId.y], Weight1Int);
#endif
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Accumulate all histograms into one.

#if BILATERAL_GRID
#	if THREADGROUP_SIZEX * THREADGROUP_SIZEY < HISTOGRAM_SIZE
#		error "Not enough threads to output complete histogram"
#	endif

	if (GroupIndex < HISTOGRAM_SIZE)
	{
		float Sum = 0;
		float SumLuminance = 0;

		LOOP for (uint y = 0; y < THREADGROUP_SIZEY; ++y)
		{
			LOOP for (uint x = 0; x < THREADGROUP_SIZEX; ++x)
			{
				float SumWeight;
				float SumLumHist;
				UnpackUINT32ToTwoFloat(SharedHistogram[GroupIndex][x][y], SumWeight, SumLumHist);
				// ComputeLogLuminanceFromHistogramPosition but include SumWeight
				float SumLum = (SumLumHist - SumWeight * EyeAdaptation_HistogramBias) / EyeAdaptation_HistogramScale;

				Sum += SumWeight;
				SumLuminance += SumLum;
			}
		}

		BilateralGridRWTexture[uint3(GroupId.xy, GroupIndex)] = float2(SumLuminance, Sum);
	}
#else	
	if (GroupIndex < HISTOGRAM_SIZE / 4) 
	{
		float4 Sum = 0;

		LOOP for (uint y = 0; y < THREADGROUP_SIZEY; ++y)
		{
			LOOP for (uint x = 0; x < THREADGROUP_SIZEX; ++x)
			{
				Sum += float4(
					SharedHistogram[GroupIndex * 4 + 0][x][y],
					SharedHistogram[GroupIndex * 4 + 1][x][y],
					SharedHistogram[GroupIndex * 4 + 2][x][y],
					SharedHistogram[GroupIndex * 4 + 3][x][y]);
			}
		}

		float2 MaxExtent = Input_ViewportSize;
		float Area = MaxExtent.x * MaxExtent.y;
	
		// Fixed to include borders.
		float NormalizeFactor = 1.0f / Area;

		// Output texture with one histogram per line, x and y unwrapped into all the lines
		HistogramRWTexture[uint2(GroupIndex, GroupId.x + GroupId.y * ThreadGroupCount.x)] = Sum * NormalizeFactor;
	}
#endif
}
