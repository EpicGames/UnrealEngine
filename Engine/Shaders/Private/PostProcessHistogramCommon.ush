// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessHistogramCommon.ush: PostProcessing histogram shared functions and structures.
=============================================================================*/

#pragma once

float EyeAdaptation_ExposureLowPercent;
float EyeAdaptation_ExposureHighPercent;
float EyeAdaptation_MinAverageLuminance;
float EyeAdaptation_MaxAverageLuminance;
float EyeAdaptation_ExposureCompensationSettings;
float EyeAdaptation_ExposureCompensationCurve;
float EyeAdaptation_DeltaWorldTime;
float EyeAdaptation_ExposureSpeedUp;
float EyeAdaptation_ExposureSpeedDown;
float EyeAdaptation_HistogramScale;
float EyeAdaptation_HistogramBias;
float EyeAdaptation_LuminanceMin;
float EyeAdaptation_BlackHistogramBucketInfluence;
float EyeAdaptation_IgnoreMaterialsEvaluationPositionBias;
float EyeAdaptation_IgnoreMaterialsLuminanceScale;
float EyeAdaptation_IgnoreMaterialsMinBaseColorLuminance;
uint EyeAdaptation_IgnoreMaterialsReconstructFromSceneColor;
float EyeAdaptation_GreyMult;
float3 EyeAdaptation_LuminanceWeights;
float EyeAdaptation_ExponentialUpM;
float EyeAdaptation_ExponentialDownM;
float EyeAdaptation_StartDistance;
float EyeAdaptation_LuminanceMax;
float EyeAdaptation_ForceTarget;
int EyeAdaptation_VisualizeDebugType;
float2 EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias;
Texture2D EyeAdaptation_ExposureCompensationCurveLUT;
SamplerState EyeAdaptation_ExposureCompensationCurveSampler;
Texture2D EyeAdaptation_MeterMaskTexture;
SamplerState EyeAdaptation_MeterMaskSampler;

float LocalExposure_HighlightContrastScale;
float LocalExposure_ShadowContrastScale;
float LocalExposure_DetailStrength;
float LocalExposure_BlurredLuminanceBlend;
float LocalExposure_MiddleGreyExposureCompensation;
float2 LocalExposure_BilateralGridUVScale;

float LocalExposure_HighlightThreshold;
float LocalExposure_ShadowThreshold;
float LocalExposure_HighlightThresholdStrength;
float LocalExposure_ShadowThresholdStrength;

float CalculateEyeAdaptationLuminance(float3 Color)
{
	return max(dot(Color, EyeAdaptation_LuminanceWeights), EyeAdaptation_LuminanceMin);
}

// inverse of ComputeLogLuminanceFromHistogramPosition
// @param LogLuminance
// @return HistogramPosition 0..1
float ComputeHistogramPositionFromLogLuminance(float LogLuminance)
{
	return LogLuminance * EyeAdaptation_HistogramScale + EyeAdaptation_HistogramBias;
}

// inverse of ComputeLuminanceFromHistogramPosition
// @param Luminance
// @return HistogramPosition 0..1
float ComputeHistogramPositionFromLuminance(float Luminance)
{
	return ComputeHistogramPositionFromLogLuminance(log2(Luminance));
}

// inverse of ComputeHistogramPositionFromLogLuminance()
// @param HistogramPosition 0..1
// @return LogLuminance
float ComputeLogLuminanceFromHistogramPosition(float HistogramPosition)
{
	return ((HistogramPosition - EyeAdaptation_HistogramBias) / EyeAdaptation_HistogramScale);
}

// inverse of ComputeHistogramPositionFromLuminance()
// @param HistogramPosition 0..1
// @return Luminance
float ComputeLuminanceFromHistogramPosition(float HistogramPosition)
{
	return exp2(ComputeLogLuminanceFromHistogramPosition(HistogramPosition));
}

#ifndef HISTOGRAM_SIZE
	#define HISTOGRAM_SIZE 64
#endif

#define HISTOGRAM_TEXEL_SIZE (HISTOGRAM_SIZE / 4)

float4 ComputeARGBStripeMaskInt(uint x)
{
	return float4(
		(x % 4) == 0,
		(x % 4) == 1,
		(x % 4) == 2,
		(x % 4) == 3);
}

float GetHistogramBucket(Texture2D HistogramTexture, uint BucketIndex)
{
	uint Texel = BucketIndex / 4;
	
	float4 HistogramColor = HistogramTexture.Load(int3(Texel, 0, 0));

	uint channel = BucketIndex % 4;
	float UnweightedValue = HistogramColor.r;
	UnweightedValue = (channel == 1) ? HistogramColor.g : UnweightedValue;
	UnweightedValue = (channel == 2) ? HistogramColor.b : UnweightedValue;
	UnweightedValue = (channel == 3) ? HistogramColor.a : UnweightedValue;

	return UnweightedValue;
}

float ComputeHistogramSum(Texture2D HistogramTexture)
{
	float Sum = 0;

	for(uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		Sum += GetHistogramBucket(HistogramTexture, i);
	}

	return Sum;
}

float GetHistogramBucket(Buffer<uint> HistogramBuffer, uint BucketIndex)
{
	return HistogramBuffer[BucketIndex];
}

float ComputeHistogramSum(Buffer<uint> HistogramBuffer)
{
	float Sum = 0;

	for (uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		Sum += GetHistogramBucket(HistogramBuffer, i);
	}

	return Sum;
}

// @param MinFractionSum e.g. ComputeHistogramSum() * 0.5f for 50% percentil
// @param MaxFractionSum e.g. ComputeHistogramSum() * 0.9f for 90% percentil
float ComputeAverageLuminanceWithoutOutlier(Texture2D HistogramTexture, float MinFractionSum, float MaxFractionSum)
{
	if (abs(EyeAdaptation_ExposureLowPercent - EyeAdaptation_ExposureHighPercent) < 0.0001f)
	{
		return 1.0f; // special case to avoid affecting existing projects (eg: Lyra)
	}

	float2 SumWithoutOutliers = 0;
	
	UNROLL for(uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		float LocalValue = GetHistogramBucket(HistogramTexture, i);

		// remove outlier at lower end
		float Sub = min(LocalValue, MinFractionSum);
		LocalValue = LocalValue - Sub;
		MinFractionSum -= Sub;
		MaxFractionSum -= Sub;
		
		// remove outlier at upper end
		LocalValue = min(LocalValue, MaxFractionSum);
		MaxFractionSum -= LocalValue;

		float LogLuminanceAtBucket = ComputeLogLuminanceFromHistogramPosition(float(i) / (float)(HISTOGRAM_SIZE-1));

		SumWithoutOutliers += float2(LogLuminanceAtBucket, 1) * LocalValue;
	}

	if (SumWithoutOutliers.y == 0)
	{
		return EyeAdaptation_MinAverageLuminance / 0.18f;
	}

	float AvgLogLuminance = SumWithoutOutliers.x / SumWithoutOutliers.y;

	return exp2(AvgLogLuminance);
}

float ComputeEyeAdaptationExposure(Texture2D HistogramTexture)
{
	const float HistogramSum = ComputeHistogramSum(HistogramTexture);
	const float AverageSceneLuminance = ComputeAverageLuminanceWithoutOutlier(HistogramTexture, HistogramSum * EyeAdaptation_ExposureLowPercent, HistogramSum * EyeAdaptation_ExposureHighPercent);
	const float LumAve = AverageSceneLuminance;

	const float ClampedLumAve = LumAve;
		
	// No longer clamping here. We are letting the target exposure be outside the min/max range, and then letting the exposure
	// gradually transion.
	return ClampedLumAve;
}

float AdaptationWeightTexture(float2 UV)
{
	return Texture2DSampleLevel(EyeAdaptation_MeterMaskTexture, EyeAdaptation_MeterMaskSampler, UV, 0).x;
}

float ExponentialAdaption(float Current, float Target, float FrameTime, float AdaptionSpeed, float M)
{
	const float Factor = 1.0f - exp2(-FrameTime * AdaptionSpeed);
	const float Value = Current + (Target - Current) * Factor * M;
	return Value;
}

float LinearAdaption(float Current, float Target, float FrameTime, float AdaptionSpeed)
{
	const float Offset = FrameTime * AdaptionSpeed;

	const float Value = (Current < Target) ? min(Target, Current + Offset) : max(Target, Current - Offset);

	return Value;
}

float ComputeEyeAdaptation(float OldExposure, float TargetExposure, float FrameTime)
{
	const float LogTargetExposure = log2(TargetExposure);
	const float LogOldExposure = log2(OldExposure);

	const float LogDiff = LogTargetExposure - LogOldExposure;

	const float AdaptionSpeed = (LogDiff > 0) ? EyeAdaptation_ExposureSpeedUp : EyeAdaptation_ExposureSpeedDown;
	const float M = (LogDiff > 0) ? EyeAdaptation_ExponentialUpM : EyeAdaptation_ExponentialDownM;

	const float AbsLogDiff = abs(LogDiff);

	// blended exposure
	const float LogAdaptedExposure_Exponential = ExponentialAdaption(LogOldExposure, LogTargetExposure, FrameTime, AdaptionSpeed, M);
	const float LogAdaptedExposure_Linear = LinearAdaption(LogOldExposure, LogTargetExposure, FrameTime, AdaptionSpeed);

	const float LogAdaptedExposure = AbsLogDiff > EyeAdaptation_StartDistance ? LogAdaptedExposure_Linear : LogAdaptedExposure_Exponential;

	// Note: no clamping here. The target exposure should always be clamped so if we are below the min or above the max,
	// instead of clamping, we will gradually transition to the target exposure. If were to clamp, the then we would have a harsh transition
	// when going from postFX volumes with different min/max luminance values.
	const float AdaptedExposure = exp2(LogAdaptedExposure);

	// for manual mode or camera cuts, just lerp to the target
	const float AdjustedExposure = lerp(AdaptedExposure,TargetExposure,EyeAdaptation_ForceTarget);
	
	return AdjustedExposure;
}

#define BILATERAL_GRID_DEPTH 32

float CalculateBaseLogLuminance(float BilateralLum, float BlurredLum, float BlurredLumBlend, float ExposureScale)
{
	return lerp(BilateralLum, BlurredLum, BlurredLumBlend) + log2(ExposureScale);
}

float CalculateBaseLogLuminance(float LogLum, float BlurredLumBlend, float ExposureScale, float2 UV, Texture3D LumBilateralGrid, Texture2D BlurredLogLum, SamplerState LumBilateralGridSampler, SamplerState BlurredLogLumSampler)
{
	float3 BilateralGridUVW;
	BilateralGridUVW.xy = UV * LocalExposure_BilateralGridUVScale;
	BilateralGridUVW.z = (ComputeHistogramPositionFromLogLuminance(LogLum) * (BILATERAL_GRID_DEPTH - 1) + 0.5f) / BILATERAL_GRID_DEPTH;
	
	float2 BilateralGridLum = Texture3DSample(LumBilateralGrid, LumBilateralGridSampler, BilateralGridUVW).xy;
	float BilateralLum = BilateralGridLum.x / BilateralGridLum.y;

	float BlurredLum = Texture2DSample(BlurredLogLum, BlurredLogLumSampler, UV).r;

	if (BilateralGridLum.y < 0.001)
	{
		// fallback to blurred luminance if bilateral grid doesn't have data
		// this can happen since grid is populated using half resolution image
		BilateralLum = BlurredLum;
	}

	return CalculateBaseLogLuminance(BilateralLum, BlurredLum, BlurredLumBlend, ExposureScale);
}

float CalculateLogLocalExposure(float LogLum, float BaseLogLum, float LogMiddleGrey, float HighlightContrastScale, float ShadowContrastScale, float DetailStrength)
{
	float DetailLogLum = LogLum - BaseLogLum;

	float BaseCentered = (BaseLogLum - LogMiddleGrey);
	float ContrastScale = BaseCentered > 0 ? HighlightContrastScale : ShadowContrastScale;

	// Apply threshold
	float ThresholdOffset = 0;
	{
		float M;
		float T;
		if (BaseCentered > 0)
		{
			M = max(0.0f, BaseCentered - LocalExposure_HighlightThreshold);
			T = smoothstep(LocalExposure_HighlightThreshold, LocalExposure_HighlightThreshold + 1.0f / (1.0f - LocalExposure_HighlightThresholdStrength), abs(BaseCentered));
		}
		else
		{
			M = min(0.0f, BaseCentered + LocalExposure_ShadowThreshold);
			T = smoothstep(LocalExposure_ShadowThreshold, LocalExposure_ShadowThreshold + 1.0f / (1.0f - LocalExposure_ShadowThresholdStrength), abs(BaseCentered));
		}

		ThresholdOffset = (BaseCentered - M) * (1 - T);

		BaseCentered -= ThresholdOffset;
	}

	float LogLocalLum = LogMiddleGrey + ThresholdOffset + BaseCentered * ContrastScale + DetailLogLum * DetailStrength;

	return LogLocalLum - LogLum;
}

float CalculateLocalExposure(float LogLum, float BaseLogLum, float LogMiddleGrey, float HighlightContrastScale, float ShadowContrastScale, float DetailStrength)
{
	return exp2(CalculateLogLocalExposure(LogLum, BaseLogLum, LogMiddleGrey, HighlightContrastScale, ShadowContrastScale, DetailStrength));
}

#define EXPOSURE_FUSION_USE_FILM_TONEMAP 1

#if EXPOSURE_FUSION_USE_FILM_TONEMAP
#include "TonemapCommon.ush"
#endif

float ExposureFusionTonemap(float Lum)
{
#if EXPOSURE_FUSION_USE_FILM_TONEMAP
	float TonemappedLuminance = FilmToneMap(Lum).r;
#else
	float TonemappedLuminance = Lum / (1 + Lum);
#endif

	TonemappedLuminance = sqrt(TonemappedLuminance);

	return TonemappedLuminance;
}

float ExposureFusionInverseTonemap(float TonemappedLuminance)
{
	TonemappedLuminance = TonemappedLuminance * TonemappedLuminance;

#if EXPOSURE_FUSION_USE_FILM_TONEMAP
	float Lum = FilmToneMapInverse(TonemappedLuminance).r;
#else
	float Lum = TonemappedLuminance / (1 - TonemappedLuminance);
#endif

	return Lum;
}

float CalculateFusionLocalExposure(float Lum, float FusionResult)
{
#if 1
	return ExposureFusionInverseTonemap(min(FusionResult, 1.0f)) / Lum;
#else
	const float LumTonemapped = ExposureFusionTonemap(Lum);
	return FusionResult / LumTonemapped;
#endif
}