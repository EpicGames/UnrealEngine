// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessHistogramReduce.usf: PostProcessing combine multiple histograms into a single one
=============================================================================*/

#include "Common.ush"

// How many lines of histograms should be compiled into one
uint LoopSize;

// The last frames eye adaptation settings
StructuredBuffer<float4> EyeAdaptationBuffer;

Texture2D InputTexture;

SamplerState InputSampler;

float2 Input_ExtentInverse;

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition: SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;

	float4 SceneColor = 0;

	// accumulate all histograms into a single one
	// could be optimized using bilinear filtering (half sample count)
	for (uint y = 0; y < LoopSize/2; ++y)
	{
		SceneColor += Texture2DSample(InputTexture, InputSampler, float2(UV.x, (float(2*y + 1) * Input_ExtentInverse.y)));
	}

	if(SvPosition.y < 1.0f)
	{
		// line 0: histogram, we're not going to divide by the number of lines because it gets normalized later anyways
		OutColor = SceneColor;
	}
	else
	{
		// line 1: store 4 channels of EyeAdaptationBuffer (copied over so we can read the value in EyeAdaptation pass which is writing to eye adaptation)

		// second line first pixel in the texture has the ExposureScale from last frame
		float4 OldExposureScale = EyeAdaptationBuffer[0];

		OutColor = OldExposureScale;
	}
}