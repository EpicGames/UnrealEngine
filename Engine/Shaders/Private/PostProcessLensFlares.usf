// Copyright Epic Games, Inc. All Rights Reserved.

#define SCENE_TEXTURES_DISABLED 1

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"

#ifndef QUADS_PER_INSTANCE
#error QUADS_PER_INSTANCE is undefined!
#endif

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

Texture2D InputTexture;
SamplerState InputSampler;
float4 FlareColor;

void LensFlareCompositePS(
	in noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float2 ScreenPos = UVAndScreenPos.zw;

	// Circular shape to mark the screen border.
	float ScreenborderMask = DiscMask(ScreenPos);

	// Extra circular shape aligned to the screen to mask big lens flares.
	ScreenborderMask *= DiscMask(ScreenPos * 0.8f);

	OutColor.rgb = Texture2DSample(InputTexture, InputSampler, UV).rgb * FlareColor.rgb * ScreenborderMask;
	OutColor.a = 0;
}

Texture2D BokehTexture;
SamplerState BokehSampler;
uint2 TileCount;
uint TileSize;
float KernelSize;
float KernelAreaInverse;
float Threshold;
float GuardBandScale;
float GuardBandScaleInverse;

float2 PixelToScreenPos(float2 PixelPos)
{
	return (PixelPos - Input_ScreenPosToViewportBias) / Input_ScreenPosToViewportScale;
}

void LensFlareBlurVS(
	uint VId : SV_VertexID,
	uint IId : SV_InstanceID,
	out noperspective float2 OutUV : TEXCOORD0,
	out noperspective float4 OutColor : TEXCOORD1,
	out float4 OutPosition : SV_POSITION)
{
	// Remap the indices to get vertexid to VId and quadid into IId
	IId = IId * QUADS_PER_INSTANCE + (VId / 6);
	VId = VId % 6;

	// Triangle A: 0:left top, 1:right top, 2: left bottom
	// Triangle B: 3:right bottom, 4:left bottom, 5: right top
	float2 LocalPos = float2(VId % 2, VId > 1 && VId < 5);
	float2 TilePos = float2(IId % TileCount.x, IId / TileCount.x) + Input_ViewportMin;

	OutUV = LocalPos.xy;

	OutPosition = float4(0, 0, 0, 1);
	OutPosition.xy = PixelToScreenPos(TilePos * TileSize) * GuardBandScaleInverse;

	float2 InputUV = clamp(Input_ExtentInverse * TilePos * TileSize, Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);

	OutColor = Texture2DSampleLevel(InputTexture, InputSampler, InputUV, 0);
	OutColor.rgb *= DiscMask(OutPosition.xy);

	const float Luminance = dot(OutColor.rgb, 1);
	const float ThresholdScale = (Luminance < Threshold) ? 0.0f : 1.0f;

	OutColor *= KernelAreaInverse;

	// Offset the corners.
	OutPosition.xy += 2.0 * (LocalPos - 0.5f) * Input_ViewportSizeInverse * KernelSize * ThresholdScale;
}

void LensFlareBlurPS(
	noperspective float2 InUV : TEXCOORD0,
	noperspective float4 InColor : TEXCOORD1,
	out float4 OutColor : SV_Target0)
{
	float4 Kernel = Texture2DSample(BokehTexture, BokehSampler, InUV);

	OutColor = float4(InColor.rgb * Kernel.rgb, 1);
}

void LensFlareCopyBloomPS(
	float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0)
{
	OutColor = InputTexture.Load(int3(SvPosition.xy, 0));
}
