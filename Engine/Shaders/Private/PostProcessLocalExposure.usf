// Copyright Epic Games, Inc. All Rights Reserved.

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessHistogramCommon.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D InputTexture;

StructuredBuffer<float4> EyeAdaptationBuffer;

Texture3D LumBilateralGrid;
Texture2D BlurredLogLum;

SamplerState TextureSampler;

RWTexture2D<float> OutputFloat;
RWTexture2D<float4> OutputFloat4;
RWTexture2D<float4> OutputFloat4_1;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void SetupLogLuminanceCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 InputPixelPos = Input_ViewportMin + DispatchThreadId;

	BRANCH
	if (any(InputPixelPos >= (uint2)Input_ViewportMax))
	{
		return;
	}

	float3 SceneColor = InputTexture.Load(uint3(InputPixelPos, 0)).rgb * View.OneOverPreExposure;
	float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor);
	OutputFloat[DispatchThreadId] = log2(LuminanceVal);
}

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void ApplyLocalExposureCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 OutputPixelPos = DispatchThreadId + Output_ViewportMin;

	if (all(OutputPixelPos < Output_ViewportMax))
	{
		const float2 UV = (DispatchThreadId + 0.5f) * Output_ExtentInverse;

		float2 ExposureScaleMiddleGrey = EyeAdaptationBuffer[0].xw;
		float ExposureScale = ExposureScaleMiddleGrey.x;
		float MiddleGreyLumValue = log2(0.18 * ExposureScaleMiddleGrey.y * LocalExposure_MiddleGreyExposureCompensation);

		float4 SceneColor = InputTexture.Load(uint3(DispatchThreadId + Input_ViewportMin, 0));

		float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor.rgb * View.OneOverPreExposure);
		float LogLuminance = log2(LuminanceVal);

		float BaseLogLum = CalculateBaseLogLuminance(LogLuminance, LocalExposure_BlurredLuminanceBlend, ExposureScale, UV, LumBilateralGrid, BlurredLogLum, TextureSampler, TextureSampler);
		float LocalExposure = CalculateLocalExposure(LogLuminance + log2(ExposureScale), BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, LocalExposure_DetailStrength);

		SceneColor.rgb *= LocalExposure;

		OutputFloat4[OutputPixelPos] = SceneColor;
	}
}

float ExposureWeight(float V, float Target)
{
	float Sigma = 0.2f;

	float A = (V - Target);
	float B = 2 * Sigma * Sigma;

	return exp2(-A * A / B);
}

float TargetLuminance;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void FusionSetupCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 OutputPixelPos = DispatchThreadId + Output_ViewportMin;

	BRANCH
	if (any(OutputPixelPos >= (uint2)Output_ViewportMax))
	{
		return;
	}

	const float2 ExposureScaleMiddleGrey = EyeAdaptationBuffer[0].xw;
	const float ExposureScale = ExposureScaleMiddleGrey.x;
	float MiddleGreyLumValue = log2(0.18 * ExposureScaleMiddleGrey.y * LocalExposure_MiddleGreyExposureCompensation);

	float3 SceneColor = InputTexture.Load(uint3(Input_ViewportMin + DispatchThreadId, 0)).rgb * View.OneOverPreExposure * ExposureScale;
	float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor);

	float HighlightLuminanceVal = LuminanceVal * LocalExposure_HighlightContrastScale;
	float ShadowLuminanceVal = LuminanceVal * LocalExposure_ShadowContrastScale;

	float LuminanceValTonemapped = ExposureFusionTonemap(LuminanceVal);
	float HighlightLuminanceValTonemapped = ExposureFusionTonemap(HighlightLuminanceVal);
	float ShadowLuminanceValTonemapped = ExposureFusionTonemap(ShadowLuminanceVal);

	float3 Weights = float3(ExposureWeight(LuminanceValTonemapped, TargetLuminance), ExposureWeight(HighlightLuminanceValTonemapped, TargetLuminance), ExposureWeight(ShadowLuminanceValTonemapped, TargetLuminance));
	Weights /= dot(Weights, 1.0f); // normalize weights

	OutputFloat4[OutputPixelPos] = float4(LuminanceValTonemapped, HighlightLuminanceValTonemapped, ShadowLuminanceValTonemapped, 0.0f);
	OutputFloat4_1[OutputPixelPos] = float4(Weights, 0.0f);
}

Texture2D WeightTexture;

SCREEN_PASS_TEXTURE_VIEWPORT(CoarserMip)
Texture2D CoarserMipTexture;
Texture2D PrevResultTexture;

FScreenTransform DispatchThreadToCoarseMipUV;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void FusionBlendCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 OutputPixelPos = DispatchThreadId + Output_ViewportMin;

	BRANCH
	if (any(OutputPixelPos >= (uint2)Output_ViewportMax))
	{
		return;
	}

	float3 Weights = WeightTexture.Load(uint3(OutputPixelPos, 0)).rgb;
	Weights /= dot(Weights, 1.0f); // normalize weights

	float3 Lums = InputTexture.Load(uint3(OutputPixelPos, 0)).rgb;
	float PrevResultVal = 0.0f;

#if LAPLACIAN
	float2 CoarserMipUV = ApplyScreenTransform(DispatchThreadId, DispatchThreadToCoarseMipUV);
	CoarserMipUV = clamp(CoarserMipUV, CoarserMip_UVViewportBilinearMin, CoarserMip_UVViewportBilinearMax);

	Lums -= Texture2DSample(CoarserMipTexture, TextureSampler, CoarserMipUV).rgb;
	PrevResultVal = Texture2DSample(PrevResultTexture, TextureSampler, CoarserMipUV).r;
#endif

	OutputFloat[OutputPixelPos] = PrevResultVal + dot(Lums, Weights);
}
