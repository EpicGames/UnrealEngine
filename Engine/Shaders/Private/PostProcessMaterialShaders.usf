// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMaterialShaders.usf: Shaders for rendering post process materials
=============================================================================*/

#include "Common.ush"
#include "ScreenPass.ush"
#include "VirtualTextureCommon.ush"

#ifndef POST_PROCESS_AR_YCBCR_CONVERSION
#define POST_PROCESS_AR_YCBCR_CONVERSION 0
#endif

#if POST_PROCESS_AR_YCBCR_CONVERSION
#include "ColorUtils.ush"

float4x4 YCbCrColorTransform;
uint YCbCrSrgbToLinear;
#endif

#pragma message("UESHADERMETADATA_VERSION 02E0E449-2223-45B4-8CD0-93AB087FBEE3")

#ifndef POST_PROCESS_MATERIAL
#define POST_PROCESS_MATERIAL 0
#endif

#define ALLOW_SUBSTRATE_PUBLIC_ACCESS (SUBSTRATE_ENABLED)

#if (POST_PROCESS_MATERIAL == 0)
#error POST_PROCESS_MATERIAL must be defined to non-zero in the shader compilation environment.
#endif

// Must match ESceneTextureId
#define PPI_PostProcessInput0 14
#define PPI_PostProcessInput1 15
#define PPI_PostProcessInput2 16
#define PPI_PostProcessInput3 17
#define PPI_PostProcessInput4 18

SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_0) 
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_1)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_2)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_3)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessInput_4)
SCREEN_PASS_TEXTURE_VIEWPORT(PostProcessOutput)

Texture2D PostProcessInput_0_Texture;
Texture2D PostProcessInput_1_Texture;
Texture2D PostProcessInput_2_Texture;
Texture2D PostProcessInput_3_Texture;
Texture2D PostProcessInput_4_Texture;

SamplerState PostProcessInput_0_Sampler;
SamplerState PostProcessInput_1_Sampler;
SamplerState PostProcessInput_2_Sampler;
SamplerState PostProcessInput_3_Sampler;
SamplerState PostProcessInput_4_Sampler;
SamplerState PostProcessInput_BilinearSampler;

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define PostProcessInput_0_SharedSampler PostProcessInput_0_Sampler
	#define PostProcessInput_1_SharedSampler PostProcessInput_0_Sampler
	#define PostProcessInput_2_SharedSampler PostProcessInput_0_Sampler
	#define PostProcessInput_3_SharedSampler PostProcessInput_0_Sampler
	#define PostProcessInput_4_SharedSampler PostProcessInput_0_Sampler
#else
	#define PostProcessInput_0_SharedSampler PostProcessInput_0_Sampler
	#define PostProcessInput_1_SharedSampler PostProcessInput_1_Sampler
	#define PostProcessInput_2_SharedSampler PostProcessInput_2_Sampler
	#define PostProcessInput_3_SharedSampler PostProcessInput_3_Sampler
	#define PostProcessInput_4_SharedSampler PostProcessInput_4_Sampler
#endif

#if MATERIAL_PATH_TRACING_BUFFER_READ
SCREEN_PASS_TEXTURE_VIEWPORT(PathTracingPostProcessInput_0)
SCREEN_PASS_TEXTURE_VIEWPORT(PathTracingPostProcessInput_1)
SCREEN_PASS_TEXTURE_VIEWPORT(PathTracingPostProcessInput_2)
SCREEN_PASS_TEXTURE_VIEWPORT(PathTracingPostProcessInput_3)
SCREEN_PASS_TEXTURE_VIEWPORT(PathTracingPostProcessInput_4)

Texture2D PathTracingPostProcessInput_0_Texture;
Texture2D PathTracingPostProcessInput_1_Texture;
Texture2D PathTracingPostProcessInput_2_Texture;
Texture2D PathTracingPostProcessInput_3_Texture;
Texture2D PathTracingPostProcessInput_4_Texture;

SamplerState PathTracingPostProcessInput_0_Sampler;
SamplerState PathTracingPostProcessInput_1_Sampler;
SamplerState PathTracingPostProcessInput_2_Sampler;
SamplerState PathTracingPostProcessInput_3_Sampler;
SamplerState PathTracingPostProcessInput_4_Sampler;
#endif

#if PATH_TRACING_POST_PROCESS_MATERIAL
// Allow users to customize the behavior of the material for the path tracing with PathTraicngQualitySwitch node
bool GetPathTracingQualitySwitch() { return true;  }
bool GetPathTracingIsShadow() { return false; }
bool GetPathTracingIsIndirectDiffuse() { return false; }
bool GetPathTracingIsIndirectSpecular() { return false; }
bool GetPathTracingIsIndirectVolume() { return false; }
#endif


#include "NeuralPostProcessCommon.usf"

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "/Engine/Generated/Material.ush"

uint ManualStencilReferenceValue;
uint ManualStencilTestMask;

struct FPostProcessMaterialVSToPS
{
	float4 Position : SV_POSITION;

#if NUM_TEX_COORD_INTERPOLATORS
	float4 TexCoords[(NUM_TEX_COORD_INTERPOLATORS+1)/2] : TEXCOORD0;
#endif
};

#if NUM_TEX_COORD_INTERPOLATORS
float2 GetUV(FPostProcessMaterialVSToPS Interpolants, int UVIndex)
{
	float4 UVVector = Interpolants.TexCoords[UVIndex / 2];
	return Mod(UVIndex, 2) == 1 ? UVVector.zw : UVVector.xy;
}

void SetUV(inout FPostProcessMaterialVSToPS Interpolants, int UVIndex, float2 InValue)
{
	FLATTEN
	if (Mod(UVIndex, 2) == 1)
	{
		Interpolants.TexCoords[UVIndex / 2].zw = InValue;
	}
	else
	{
		Interpolants.TexCoords[UVIndex / 2].xy = InValue;
	}
}
#endif

FMaterialVertexParameters GetPostProcessMaterialVSParameters(float2 UV)
{
	// Most params irrelevant so not a lot to fill out here
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.VertexColor = 1.f.xxxx;
	Result.WorldPosition = float3(UV, 0.f); 
	
#if NUM_MATERIAL_TEXCOORDS_VERTEX
	UNROLL
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = UV;
	}
#endif

	return Result;
}

#if (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)

#if VERTEXSHADER

void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	out FPostProcessMaterialVSToPS Output
	)
{
	Output = (FPostProcessMaterialVSToPS)0;
	DrawRectangle(InPosition, Output.Position);

#if NUM_TEX_COORD_INTERPOLATORS
	FMaterialVertexParameters VertexParameters = GetPostProcessMaterialVSParameters(InPosition.xy);
	 
	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);
	
	{
		UNROLL
		for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
		{
			SetUV(Output, CoordinateIndex, InPosition.xy);
		}
	}
	{
		UNROLL
		for (int CoordinateIndex = NUM_MATERIAL_TEXCOORDS; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
		{
			SetUV(Output, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
		}
	}
#endif
}

void MainVS_VideoOverlay(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUV);
}

#elif PIXELSHADER

#define STENCIL_TEST_EQUAL		0x1u
#define STENCIL_TEST_LESS		0x2u
#define STENCIL_TEST_GREATER	0x4u

// Perform the stencil test manually by discarding test failure.
// This is necessary when Nanite writes custom stencil and the platform cannot support writing arbitrary values directly to
// the custom stencil buffer per-pixel
void ManualStencilTest(uint2 PixelPos)
{
	const uint Stencil = CalcSceneCustomStencil(PixelPos);
	const uint Test =
		(ManualStencilReferenceValue == Stencil	? STENCIL_TEST_EQUAL	: 0) |
		(ManualStencilReferenceValue <  Stencil	? STENCIL_TEST_LESS		: 0) |
		(ManualStencilReferenceValue >  Stencil	? STENCIL_TEST_GREATER	: 0);

	if ((Test & ManualStencilTestMask) == 0)
	{
		discard;
	}
}

void MainPS(
	in FPostProcessMaterialVSToPS Input,
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();
	FMaterialPixelParameters Parameters = MakeInitializedMaterialPixelParameters();
	FPixelMaterialInputs PixelMaterialInputs;

	// can be optimized
	float4 SvPosition = Input.Position;
	float2 ViewportUV = (SvPosition.xy - PostProcessOutput_ViewportMin.xy) * PostProcessOutput_ViewportSizeInverse.xy;

#if MANUAL_STENCIL_TEST
	ManualStencilTest((uint2)SvPosition.xy);
#endif

#if NUM_TEX_COORD_INTERPOLATORS
	{
		UNROLL
		for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
		{
			Parameters.TexCoords[CoordinateIndex] = ViewportUV;
		}
	}
	{
		UNROLL
		for (int CoordinateIndex = NUM_MATERIAL_TEXCOORDS; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
		{
			Parameters.TexCoords[CoordinateIndex] = GetUV(Input, CoordinateIndex);
		}
	}
#endif

	Parameters.VertexColor = 1;
	
	SvPosition.z = LookupDeviceZ(ViewportUVToBufferUV(ViewportUV));
	SvPosition.z = max(SvPosition.z, 1e-18);

	// fill out other related material parameters
	CalcMaterialParametersPost(Parameters, PixelMaterialInputs, SvPosition, ViewportUV, true);

#if NEURAL_POSTPROCESS_PREPASS && MATERIAL_NEURAL_POST_PROCESS

	OutColor = 0.5f;

#if HAVE_GetNeuralInput1

	float4 Index = GetNeuralInput0(Parameters);
	float3 NeuralInput0 = GetNeuralInput1(Parameters).rgb;
	float  Mask = GetNeuralInput2(Parameters);
	const bool bWriteToBuffer = Index.x >= 0; //Configuration the same across all pixels.

	// Update the source type so we can copy on demand
	BRANCH
	if (all((uint2)SvPosition.xy == uint2(0, 0)))
	{
		const int SourceType = bWriteToBuffer ? SOURCE_TYPE_BUFFER : SOURCE_TYPE_TEXTURE;
		UpdateNeuralProfileSourceType(SourceType);
	}

	// Write to buffer/texture based on config
	BRANCH
	if (Mask != 0)
	{
		BRANCH
		if (bWriteToBuffer)
		{
			int Batch = (int)Index.x;
			int StartChannel = (int)Index.y;
			float2 BufferWH = Index.zw;
			SaveToNeuralBuffer(Batch, StartChannel, BufferWH, NeuralInput0);
		}
		else
		{
			int StartChannel = (int)Index.y;
			float2 CustomViewportUV = Index.zw;
			float2 Position = CustomViewportUV * PostProcessOutput_ViewportSize.xy + PostProcessOutput_ViewportMin.xy;
			if (StartChannel >= 0 && StartChannel <= 2)
			{
				RWNeuralTexture[(uint2)Position.xy] = float4(NeuralInput0, 0.0f);
			}
		}
	}
	
	// This is for debug only
	OutColor.xyz = NeuralInput0;
#if !MATERIALBLENDING_MODULATE && POST_PROCESS_MATERIAL_BEFORE_TONEMAP && !POST_PROCESS_DISABLE_PRE_EXPOSURE_SCALE
	OutColor.xyz *= View.PreExposure;
#endif

	return;

#endif // HAVE_GetNeuralInput1
#endif // NEURAL_POSTPROCESS_PREPASS	

	// In Unreal, an alpha channel value of 0.0f means that the pixel is opaque and a value of 1.0f means that there is translucency.
	// This is the opposite of how the alpha channel is interpreted in formats like PNG. For this reason, rendered images that are exported
	// from the engine will have their alpha channel inverted to match the PNG convention. However, for consistency inside the rendering chain,
	// we need to stick to 0.0f as the default value.
	float Alpha = 0.0f;
	float3 EmissiveColor = 0.0f;
#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	// Post process node forces a single unlit BSDF. We get the raw data from the Substrate post process node from the raw BSDF on the tree.
	FSubstratePixelHeader SubstratePixelHeader = Parameters.GetFrontSubstrateHeader();
	FSubstrateBSDF UnlitBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];
	UnlitBSDF.SubstrateSanitizeBSDF(); // required wehen not calling SubstrateUpdateTreeUnlit

	half3 PPColor = BSDF_GETEMISSIVE(UnlitBSDF);
	half  PPAlpha = saturate(1.0 - UNLIT_TRANSMITTANCE(UnlitBSDF).x);	// The current interface only allows legacy opacity transformed to transmittance as 1-Opacity. See UMaterialExpressionSubstratePostProcess.

	// Substrate only use premultiplied alpha blending state (except for modulate and holdout) so we need to convert to the here.
#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	EmissiveColor = PPColor;
#elif MATERIALBLENDING_ALPHACOMPOSITE
	EmissiveColor = PPColor;	// Premultipled alpha already baked in by the user
#elif MATERIALBLENDING_ALPHAHOLDOUT
	EmissiveColor = 0.0;
#elif MATERIALBLENDING_TRANSLUCENT
	EmissiveColor = PPColor * PPAlpha;
#elif MATERIALBLENDING_ADDITIVE
	EmissiveColor = PPColor;	// not affected by alpha
#elif MATERIALBLENDING_MODULATE
	EmissiveColor = PPColor;	// not affected by alpha
#else
	EmissiveColor = PPColor;
#endif

#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
	PPAlpha = GetMaterialOpacity(PixelMaterialInputs);
#endif

#if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
	Alpha = PPAlpha;
#endif

#else // SUBSTRATE_ENABLED

	// Grab emissive colour as output
  #if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
	Alpha = GetMaterialOpacity(PixelMaterialInputs);
  #endif
	EmissiveColor = GetMaterialEmissive(PixelMaterialInputs);
#endif // SUBSTRATE_ENABLED

#ifdef POST_PROCESS_PROPAGATE_ALPHA_INPUT
	Alpha = SceneTextureLookup(float2(0,0), PPI_PropagatedAlpha, false).r;
#endif

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(Parameters.VirtualTextureFeedback, Parameters.SvPosition, View.VTFeedbackBuffer);
#endif

	OutColor = float4(EmissiveColor, Alpha );

#if !MATERIALBLENDING_MODULATE && POST_PROCESS_MATERIAL_BEFORE_TONEMAP && !POST_PROCESS_DISABLE_PRE_EXPOSURE_SCALE
	OutColor.xyz *= View.PreExposure;
#endif
} 

void MainPS_VideoOverlay(
	in float2 InUV : TEXCOORD0,
	in float4 SvPosition : SV_Position,		// after all interpolators
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();
	FMaterialPixelParameters Parameters = MakeInitializedMaterialPixelParameters();
	FPixelMaterialInputs PixelMaterialInputs;

	float2 ViewportUV = InUV;

#if NUM_MATERIAL_TEXCOORDS
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_MATERIAL_TEXCOORDS;CoordinateIndex++)
	{
		Parameters.TexCoords[CoordinateIndex] = ViewportUV;
	}
#endif

	Parameters.VertexColor = 1;

	SvPosition.z = LookupDeviceZ(ViewportUVToBufferUV(ViewportUV));
	SvPosition.z = max(SvPosition.z, 1e-18);

	// fill out other related material parameters
	CalcMaterialParametersPost(Parameters, PixelMaterialInputs, SvPosition, ViewportUV, true);

	// Grab emissive colour as output
#if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
	const float Alpha = GetMaterialOpacity(PixelMaterialInputs);
#else
	// In Unreal, an alpha channel value of 0.0f means that the pixel is opaque and a value of 1.0f means that there is translucency.
	// This is the opposite of how the alpha channel is interpreted in formats like PNG. For this reason, rendered images that are exported
	// from the engine will have their alpha channel inverted to match the PNG convention. However, for consistency inside the rendering chain,
	// we need to stick to 0.0f as the default value.
	const float Alpha = 0.0f;
#endif
	OutColor = float4(GetMaterialEmissive(PixelMaterialInputs), Alpha );
}

#else // !VERTEXSHADER && !PIXELSHADER
#error Wrong shader domain.

#endif

#else // FEATURE_LEVEL > FEATURE_LEVEL_ES3_1

//
// Mobile version
//
void MainVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out FPostProcessMaterialVSToPS Output
	)
{
	Output = (FPostProcessMaterialVSToPS)0;
	float2 OutUV;
	DrawRectangle(InPosition, InTexCoord, Output.Position, OutUV);
			
#if NUM_TEX_COORD_INTERPOLATORS
	FMaterialVertexParameters VertexParameters = GetPostProcessMaterialVSParameters(InPosition.xy);

	float2 CustomizedUVs[NUM_TEX_COORD_INTERPOLATORS];
	GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
	GetCustomInterpolators(VertexParameters, CustomizedUVs);

	{
		UNROLL
		for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
		{
		#if POST_PROCESS_AR_PASSTHROUGH
			SetUV(Output, CoordinateIndex, OutUV);
		#else
			SetUV(Output, CoordinateIndex, InPosition.xy);
		#endif
		}
	}
	{
		UNROLL
		for (int CoordinateIndex = NUM_MATERIAL_TEXCOORDS; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
		{
			SetUV(Output, CoordinateIndex, CustomizedUVs[CoordinateIndex]);
		}
	}
#endif
}

void MainPS(
	in FPostProcessMaterialVSToPS Input,
	out half4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();
	FMaterialPixelParameters Parameters = MakeInitializedMaterialPixelParameters();
	FPixelMaterialInputs PixelMaterialInputs;

	// can be optimized
	float4 SvPosition = Input.Position;
	float2 ViewportUV = (SvPosition.xy - PostProcessOutput_ViewportMin.xy) * PostProcessOutput_ViewportSizeInverse.xy;
	float2 BufferUV = ViewportUVToBufferUV(ViewportUV);
	
#if NUM_TEX_COORD_INTERPOLATORS
	{
		UNROLL
		for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
		{
		#if POST_PROCESS_AR_PASSTHROUGH
			Parameters.TexCoords[CoordinateIndex] = GetUV(Input, CoordinateIndex);
		#else
			Parameters.TexCoords[CoordinateIndex] = ViewportUV;
		#endif
		}
	}
	{
		UNROLL
		for (int CoordinateIndex = NUM_MATERIAL_TEXCOORDS; CoordinateIndex < NUM_TEX_COORD_INTERPOLATORS; CoordinateIndex++)
		{
			Parameters.TexCoords[CoordinateIndex] = GetUV(Input, CoordinateIndex);
		}
	}
#endif
	Parameters.VertexColor = 1;
		
	float ClipValue = 1.0f;

	float DeviceZ = LookupDeviceZ(BufferUV);
	SvPosition.z = DeviceZ;
	SvPosition.z = max(SvPosition.z, 1e-18);
	// fill out other related material parameters
	CalcMaterialParametersPost(Parameters, PixelMaterialInputs, SvPosition, ViewportUV, true);

	float Alpha = 0.0f;
	float3 EmissiveColor = 0.0f;
#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	// Unlit forces a single BSDF
	FSubstrateBSDF UnlitBSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	UnlitBSDF.SubstrateSanitizeBSDF();

	half3 PPColor = BSDF_GETEMISSIVE(UnlitBSDF);
	half  PPAlpha = saturate(1.0 - UNLIT_TRANSMITTANCE(UnlitBSDF).x);	// The current interface only allows legacy opacity transformed to transmittance as 1-Opacity. See UMaterialExpressionSubstratePostProcess.

	// Substrate only use premultiplied alpha blending state (except for modulate and holdout) so we need to convert to the here.
#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	EmissiveColor = PPColor;
#elif MATERIALBLENDING_ALPHACOMPOSITE
	EmissiveColor = PPColor;	// Premultipled alpha already baked in by the user
#elif MATERIALBLENDING_ALPHAHOLDOUT
	EmissiveColor = 0.0;
#elif MATERIALBLENDING_TRANSLUCENT
	EmissiveColor = PPColor * PPAlpha;
#elif MATERIALBLENDING_ADDITIVE
	EmissiveColor = PPColor;	// not affected by alpha
#elif MATERIALBLENDING_MODULATE
	EmissiveColor = PPColor;	// not affected by alpha
#else
	EmissiveColor = PPColor;
#endif

#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
	PPAlpha = GetMaterialOpacity(PixelMaterialInputs);
#endif

#if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
	Alpha = PPAlpha;
#endif

#else // SUBSTRATE_ENABLED

	// Grab emissive colour as output
	EmissiveColor = GetMaterialEmissive(PixelMaterialInputs);
	
#if POST_PROCESS_AR_YCBCR_CONVERSION
	const float3 YCbCr = EmissiveColor.yzx;
	EmissiveColor.rgb = YuvToRgb(YCbCr, YCbCrColorTransform, YCbCrSrgbToLinear);
#endif
	
  #if MATERIAL_OUTPUT_OPACITY_AS_ALPHA
	Alpha = GetMaterialOpacity(PixelMaterialInputs);
  #else
	Alpha = 0.0f;
  #endif

#endif // SUBSTRATE_ENABLED

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(Parameters.VirtualTextureFeedback, Parameters.SvPosition, View.VTFeedbackBuffer);
#endif

	half4 FullSceneColor = half4(EmissiveColor, Alpha);
	   
#if POST_PROCESS_MATERIAL_BEFORE_TONEMAP
#if OUTPUT_GAMMA_SPACE
	FullSceneColor.rgb = sqrt(FullSceneColor.rgb);
#endif

#if !MATERIALBLENDING_MODULATE && !POST_PROCESS_DISABLE_PRE_EXPOSURE_SCALE
	FullSceneColor.xyz *= View.PreExposure;
#endif
#endif
	OutColor = FullSceneColor;
}
#endif // FEATURE_LEVEL > FEATURE_LEVEL_ES3_1