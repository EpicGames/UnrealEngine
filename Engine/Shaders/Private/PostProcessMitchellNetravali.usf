// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDownsample.usf: PostProcessing down sample.
=============================================================================*/

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "Common.ush"
#include "ScreenPass.ush"
#include "EyeAdaptationCommon.ush"

//------------------------------------------------------- COMPILE TIME CONSTANTS

#define THREADGROUP_SIZEX 8
#define THREADGROUP_SIZEY 8


static const int2 kOffsetsCross3x3[4] = {
	int2(-1, -1),
	int2( 1, -1),
	int2(-1,  1),
	int2( 1,  1),
};


//------------------------------------------------------- FUNCTIONS

// Faster but less accurate luma computation. 
// Luma includes a scaling by 4.
float Luma4(float3 Color)
{
	return (Color.g * 2.0) + (Color.r + Color.b);
}

// Optimized HDR weighting function.
float HdrWeight4(float3 Color, float Exposure) 
{
	return rcp(Luma4(Color) * Exposure + 4.0);
}

// Mitchell Netravali
// B-spline:       B=1      C=0
// Mitchell:       B=1/3    C=1/3
// Catmull-Rom:    B=0      C=1/2
// Robidoux:       B=0.3782 C=0.3109 (cylindrical)
// Robidoux Sharp: B=0.2620 C=0.3690 (cylindrical)
// Robidoux Soft:  B=0.6796 C=0.1602 (cylindrical)
void MitchellNetravaliCoefs(const float B, const float C, out float OutQ0[4], out float OutQ1[4])
{
	OutQ0[0] = (6.0 - 2.0 * B) / 6.0;
	OutQ0[1] = 0.0;
	OutQ0[2] = (-18.0 + 12.0 * B + 6.0 * C) / 6.0;
	OutQ0[3] = (12.0 - 9.0 * B - 6.0 * C) / 6.0;

	OutQ1[0] = (8 * B + 24 * C) / 6.0;
	OutQ1[1] = (-12 * B - 48 * C) / 6.0;
	OutQ1[2] = (6 * B + 30 * C) / 6.0;
	OutQ1[3] = (-B - 6 * C) / 6.0;
}

float MitchellNetravali(float d, const float B, const float C)
{
	// Coeficient ends up known at compile time.
	float Q0[4];
	float Q1[4];
	MitchellNetravaliCoefs(B, C, Q0, Q1);

	if (d < 1)
	{
		return Q0[0] + d * (Q0[1] + d * (Q0[2] + d * Q0[3]));
	}
	else if ((d >= 1) && (d < 2))
	{
		return Q1[0] + d * (Q1[1] + d * (Q1[2] + d * Q1[3]));
	}
	else
	{
		return 0;
	}
}


Texture2D InputTexture;
SamplerState InputSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

float2 DispatchThreadToInputUVScale;
float2 DispatchThreadToInputUVBias;

RWTexture2D<float4>	OutputTexture;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void DownsampleMainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const float FrameExposureScale = ToScalarMemory(EyeAdaptationLookup() * View.OneOverPreExposure);

	const float B = rcp(3.0);
	const float C = rcp(3.0);

	float2 InputBufferUV = DispatchThreadId * DispatchThreadToInputUVScale + DispatchThreadToInputUVBias;

	float4 OutColor = 0;
	float OutWeight = 0;

	{
		const uint KernelSize = 4;

		// Position of the output pixel in the input buffer.
		float2 OutputPixelPos = InputBufferUV * Input_Extent;

		float2 TopLeftKernel = floor(OutputPixelPos + (0.5 - 0.5 * KernelSize)) + 0.5;

		UNROLL
		for (uint x = 0; x < KernelSize; x++)
		{
			UNROLL
			for (uint y = 0; y < KernelSize; y++)
			{
				float2 SamplePixelPos = TopLeftKernel + float2(x, y);

				float2 SampleUV = SamplePixelPos * Input_ExtentInverse;

				SampleUV = clamp(SampleUV, Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);

				float4 SampleColor = InputTexture.SampleLevel(InputSampler, SampleUV, 0);

				float2 PixelOffset = abs(SamplePixelPos - OutputPixelPos);

				float Weight = MitchellNetravali(PixelOffset.x, B, C) * MitchellNetravali(PixelOffset.y, B, C);
				
				float HdrWeight = HdrWeight4(SampleColor.rgb, FrameExposureScale);

				OutColor += (HdrWeight * Weight) * SampleColor;
				OutWeight += HdrWeight * Weight;
			}
		}
	}
	
	if (OutWeight > 0)
	{
		OutColor *= rcp(OutWeight);
	}

	// Remove unecessary ALU work for alpha channel if unsupported on project.
	#if DIM_ALPHA_CHANNEL
	{
		OutColor[3] = select(OutColor[3] > 0.995, 1.0, OutColor[3]);
		OutColor[3] = select(OutColor[3] < 0.005, 0.0, OutColor[3]);
	}
	#else
		OutColor.a = 0;
	#endif

	if (all(DispatchThreadId < Input_ViewportMax))
	{
		OutputTexture[DispatchThreadId] = OutColor;
	}
}
 