// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMobile.usf: Combined {bloom, sunshafts, depth of field}
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "PostprocessHistogramCommon.ush"
#include "ScreenPass.ush"

Texture2DMultiView SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D LastFrameSceneColorTexture;
SamplerState LastFrameSceneColorSampler;

Texture2D SunShaftAndDofTexture;
SamplerState SunShaftAndDofSampler;

Texture2D DofNearTexture;
SamplerState DofNearSampler;

Texture2D DofDownTexture;
SamplerState DofDownSampler;

Texture2D DofBlurTexture;
SamplerState DofBlurSampler;

Texture2DMultiView BloomDownSourceTexture;
SamplerState BloomDownSourceSampler;

Texture2DMultiView BloomUpSourceATexture;
SamplerState BloomUpSourceASampler;

Texture2DMultiView BloomUpSourceBTexture;
SamplerState BloomUpSourceBSampler;

Texture2D SunAlphaTexture;
SamplerState SunAlphaSampler;

Texture2D SunBlurTexture;
SamplerState SunBlurSampler;

Texture2DMultiView BloomSetup_BloomTexture;
SamplerState BloomSetup_BloomSampler;

Texture2DMultiView BloomUpTexture;
SamplerState BloomUpSampler;

Texture2D SunMergeTexture;
SamplerState SunMergeSampler;

Texture2D LastFrameSunMergeTexture;
SamplerState LastFrameSunMergeSampler;

float4 BufferSizeAndInvSize;
float4 DofBlurSizeAndInvSize;
float4 BufferASizeAndInvSize;
float4 BufferBSizeAndInvSize;
float4 BufferViewRect;

// Point on circle.
float2 Circle(float Start, float Points, float StepIndex) 
{
	float Rad = (3.141592 * 2.0 * (1.0 / Points)) * (StepIndex + Start);
	return float2(sin(Rad), cos(Rad));
}

float BloomMaxBrightness;
float BloomThreshold;

half FocusDistFar()
{
	return View.DepthOfFieldFocalDistance + View.DepthOfFieldFocalRegion;
}

half FocusDistNear()
{
	return View.DepthOfFieldFocalDistance;
}

// Alpha = 0.5 is full size, >0.5 rate at which near and far hit maximum.
float4 SunColorApertureDiv2;

// Returns 0=max near DOF, 0.5=in focus, 1.0=max far DOF
half Coc(half Depth) 
{
	half FocusDist = clamp(Depth,half(FocusDistNear()),half(FocusDistFar()));
	half CocValue = ((Depth - FocusDist) / Depth);
	return saturate(CocValue * SunColorApertureDiv2.a + 0.5);
}


//////////////////////////


float ComputeDOFNearFocalMask(float SceneDepth)
{
	float NearFocalPlane = View.DepthOfFieldFocalDistance;

	return saturate((NearFocalPlane - SceneDepth) / View.DepthOfFieldNearTransitionRegion);
}

// todo move to central place
float ComputeDOFFarFocalMask(float SceneDepth)
{
	float FarFocalPlane = View.DepthOfFieldFocalDistance + View.DepthOfFieldFocalRegion;

	return saturate((SceneDepth - FarFocalPlane) / View.DepthOfFieldFarTransitionRegion);
}

// Returns 0=max near DOF, 0.5=in focus, 1.0=max far DOF
half Coc2(half Depth)
{
	half N = ComputeDOFNearFocalMask(Depth);
	half F = ComputeDOFFarFocalMask(Depth);
	if (F > N)
	{
		return (F * 0.5) + 0.5;
	}
	return (1.0-N) * 0.5;
}

//////////////////////////////////
half2 SunConstDepthMaskScaleBias() 
{
	half DepthMin = 65504.0 - 16384.0;
	half DepthMax = 65504.0 - 0.0;
	// Compute scale and bias.
	half Scale = 1.0/(DepthMax-DepthMin);
	return half2(Scale,-DepthMin * Scale);
}

half4 SampleMultiViewTexture(Texture2DMultiView Texture, SamplerState Sampler, float2 SceneUV, float ArrayIndex)
{
#if MOBILE_MULTI_VIEW
	return Texture.Sample(Sampler, float3(SceneUV.xy,ArrayIndex));
#else
	return Texture.Sample(Sampler, SceneUV);
#endif
}

float4 SampleSceneColor(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(SceneColorTexture, SceneColorSampler, SceneUV.xy, ArrayIndex);
}

half4 SampleBloomDown(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(BloomDownSourceTexture, BloomDownSourceSampler, SceneUV.xy, ArrayIndex);
}

half4 SampleBloomUpA(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(BloomUpSourceATexture, BloomUpSourceASampler, SceneUV.xy, ArrayIndex);
}

// Sampling BloomUpB requires float return type to avoid precision issues leading to artifacts
// e.g. blue tint all over the view when looking partially at the sky.
float4 SampleBloomUpB(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(BloomUpSourceBTexture, BloomUpSourceBSampler, SceneUV.xy, ArrayIndex);
}

half4 SampleBloomSetup(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(BloomSetup_BloomTexture, BloomSetup_BloomSampler, SceneUV.xy, ArrayIndex);
}

half4 SampleBloomUp(float2 SceneUV, float ArrayIndex)
{
	return SampleMultiViewTexture(BloomUpTexture, BloomUpSampler, SceneUV.xy, ArrayIndex);
}

//
// Convert depth in alpha into combined circle of confusion and sun intensity.
//

#if SHADER_SUN_MASK

void SunMaskPS_Mobile(
	float4 InUVPos : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF_TYPE OutSunShaftAndDof : SV_Target0
#if MOBILE_USESUN && METAL_MSAA_HDR_DECODE
	, out HALF4_TYPE OutColor : SV_Target1
#endif
	)
{
	half4 SceneColor = SampleSceneColor(InUVPos.xy,GetEyeIndex(StereoInput));

	#if MOBILE_USEDEPTHTEXTURE 
		half InDepth = ConvertFromDeviceZ(MobileSceneTextures.SceneDepthTexture.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, InUVPos.xy, 0).r);
	#else
		half InDepth = ConvertFromDeviceZ(MobileSceneTextures.SceneDepthAuxTexture.SampleLevel(MobileSceneTextures.SceneDepthAuxTextureSampler, InUVPos.xy, 0).r);
	#endif
	
	#if MOBILE_USESUN

		#if METAL_MSAA_HDR_DECODE
			SceneColor.rgb *= rcp(SceneColor.r*(-0.299) + SceneColor.g*(-0.587) + SceneColor.b*(-0.114) + 1.0);
			OutColor = SceneColor;
		#endif

		half2 DepthMaskScaleBias = SunConstDepthMaskScaleBias();
		half FarAmount = saturate(InDepth * DepthMaskScaleBias.x + DepthMaskScaleBias.y);
		half3 SunAmount = SceneColor.rgb * SunColorApertureDiv2.rgb;

		float SunLuminance = max(Luminance(SunAmount), 6.10352e-5);
		float AdjustedLuminance = clamp(SunLuminance - BloomThreshold, 0.0f, BloomMaxBrightness);
		SunAmount = SunAmount / SunLuminance * AdjustedLuminance * 2.0f;

	
		half2 Pos = InUVPos.zw * 0.5 + 0.5;
		half EdgeMask = 1.0f - Pos.x * (1.0f - Pos.x) * Pos.y * (1.0f - Pos.y) * 8.0f;
		EdgeMask = EdgeMask * EdgeMask;
	
		FarAmount *= 1.0-EdgeMask;
	
		OutSunShaftAndDof = min(min(SunAmount.r, SunAmount.g), SunAmount.b) * FarAmount;
	#else
		OutSunShaftAndDof = 0.0;
	#endif

	#if MOBILE_USEDOF
		OutSunShaftAndDof += Coc2(InDepth);
	#endif
}

#endif

//
// Pre-tonemap before hardware box-filtered resolve.
//

void PreTonemapMSAA_Mobile(
	float4 InUVPos : TEXCOORD0,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
#if (METAL_ES3_1_PROFILE && !MAC)
	// On-chip pre-tonemap before MSAA resolve.
	OutColor = SubpassFetchRGBA_0();
	OutColor.rgb *= rcp(OutColor.r*0.299 + OutColor.g*0.587 + OutColor.b*0.114 + 1.0);
#endif
}

#if MOBILE_MULTI_VIEW
	Texture2DArray InputTexture;
#else
	Texture2D InputTexture;
#endif
SamplerState InputSampler;

float4 SampleInput(float2 SceneUV, float ArrayIndex)
{
#if MOBILE_MULTI_VIEW
	return Texture2DArraySample(InputTexture, InputSampler, float3(SceneUV.xy,ArrayIndex));
#else
	return Texture2DSample(InputTexture, InputSampler, SceneUV);
#endif
}

void MSAADecodeAndCopyRectPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
#if (METAL_ES3_1_PROFILE && !MAC)
	float2 UV = UVAndScreenPos.xy;
	OutColor = Texture2DSample(InputTexture, InputSampler, UV);
	OutColor.rgb *= rcp(OutColor.r*(-0.299) + OutColor.g*(-0.587) + OutColor.b*(-0.114) + 1.0);
#endif
}

// 
// Bloom Setup - Mask Bloom and Downsample 1/16 Area
//

void BloomVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords[2] : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{	
	StereoSetupVS(StereoInput, StereoOutput);
	float2 TransformedUV;
	DrawRectangle(InPosition, InTexCoord, OutPosition, TransformedUV);

	OutTexCoords[0] = TransformedUV + BufferSizeAndInvSize.zw * float2(-1, -1);
	OutTexCoords[1] = TransformedUV + BufferSizeAndInvSize.zw * float2( 1,  1);
}



float Luma4(float3 Color)
{
	return (Color.g * 2.0) + (Color.r + Color.b);
}

float HdrWeight4(float3 Color)
{
	return rcp(Luma4(Color) + 4.0);
}

float HdrWeightInv4(float3 Color)
{
	return 4.0 * rcp(1.0 - Luma4(Color));
}

void BloomPS_Mobile(
	float2 InUVs[2] : TEXCOORD0,
	in FStereoPSInput StereoInput
#if MOBILE_USEBLOOM

	, out HALF4_TYPE OutColor : SV_Target0

	#if MOBILE_USEDOF || MOBILE_USESUN

		,out HALF_TYPE OutSunShaftAndDof : SV_Target1

		#if MOBILE_USEEYEADAPTATION
			, out HALF_TYPE OutEyeAdaptation : SV_Target2
		#endif

	#else

		#if MOBILE_USEEYEADAPTATION
			, out HALF_TYPE OutEyeAdaptation : SV_Target1
		#endif

	#endif


#else

	#if MOBILE_USEDOF || MOBILE_USESUN

		, out HALF_TYPE OutSunShaftAndDof : SV_Target0

		#if MOBILE_USEEYEADAPTATION
			, out HALF_TYPE OutEyeAdaptation : SV_Target1
		#endif

	#else

		#if MOBILE_USEEYEADAPTATION
			, out HALF_TYPE OutEyeAdaptation : SV_Target0
		#endif

	#endif

#endif
	)
{
	half3 AverageColor = 0.0f;

	// Need UV clamping in case the viewport rect is at an odd size or pixel alignment.  This can be the case even if the
	// final output rect is even pixel alignment, factoring in dynamic resolution scaling.
	float2 ClampedUVs[4];
	ClampedUVs[0] = max(InUVs[0], BufferViewRect.xy);		// -1, -1
	ClampedUVs[1] = min(InUVs[1], BufferViewRect.zw); //  1   1
	ClampedUVs[2] = float2(ClampedUVs[1].x, ClampedUVs[0].y); //  1  -1
	ClampedUVs[3] = float2(ClampedUVs[0].x, ClampedUVs[1].y); // -1,  1
	
#if MOBILE_USEBLOOM || MOBILE_USESUN || MOBILE_USEEYEADAPTATION
	
	float4 C0 = SampleSceneColor(ClampedUVs[0],GetEyeIndex(StereoInput));
	float4 C1 = SampleSceneColor(ClampedUVs[1],GetEyeIndex(StereoInput));
	float4 C2 = SampleSceneColor(ClampedUVs[2],GetEyeIndex(StereoInput));
	float4 C3 = SampleSceneColor(ClampedUVs[3],GetEyeIndex(StereoInput));

	C0 *= HdrWeight4(C0);
	C1 *= HdrWeight4(C1);
	C2 *= HdrWeight4(C2);
	C3 *= HdrWeight4(C3);

	// Output color is average.
	AverageColor.rgb = (C0.rgb * 0.25) + (C1.rgb * 0.25) + (C2.rgb * 0.25) + (C3.rgb * 0.25);
	AverageColor.rgb *= HdrWeightInv4(AverageColor);

	#if METAL_MSAA_HDR_DECODE
		// This should really happen before the average, instead doing after average as optimization.
		AverageColor.rgb *= rcp(AverageColor.r*(-0.299) + AverageColor.g*(-0.587) + AverageColor.b*(-0.114) + 1.0);
	#endif

	// Try to kill negatives and NaNs here
	AverageColor.rgb = max(AverageColor.rgb, 0);

	
	#if MOBILE_USEBLOOM || MOBILE_USESUN
		// Trim bloom and sunshafts black level.
		half TotalLuminance = Luminance(AverageColor.rgb);
		half BloomLuminance = TotalLuminance - BloomThreshold;
		half Amount = saturate(BloomLuminance * 0.5f);
	#endif

	#if MOBILE_USEBLOOM 
		OutColor.rgb = AverageColor;
		OutColor.rgb *= Amount;
		OutColor.a = 0;
	#endif
#endif

#if MOBILE_USEDOF || MOBILE_USESUN
	half A0 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, ClampedUVs[0]).r;
	half A1 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, ClampedUVs[1]).r;
	half A2 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, ClampedUVs[2]).r;
	half A3 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, ClampedUVs[3]).r;
#endif

// In the case of both DOF and SUN,
// Split out alpha back into dual components (circle of confusion size and sun amount).
// Expand {near to in-focus} {0.0 to 0.5} to {0.0 to 1.0} for near DOF diolation.
// Must keep 1.0 the in-focus here (sunshaft pass will use this data).

#if MOBILE_USEDOF
	// Expand {near to in-focus} {0.0 to 0.5} to {0.0 to 1.0} for near DOF diolation.
	// Must keep 1.0 the in-focus here (sunshaft pass will use this data).
	half Coc0 = saturate(A0*2.0);
	half Coc1 = saturate(A1*2.0);
	half Coc2 = saturate(A2*2.0);
	half Coc3 = saturate(A3*2.0);

	// Take min of COC (which is maximum near radius).
	OutSunShaftAndDof = min(min(Coc0,Coc1),min(Coc2,Coc3));

	// Improve the quality of near diolation.
	OutSunShaftAndDof = 1.0 - OutSunShaftAndDof;
	OutSunShaftAndDof *= OutSunShaftAndDof;
	OutSunShaftAndDof = 1.0 - OutSunShaftAndDof;
#elif MOBILE_USESUN
	OutSunShaftAndDof = 0.0f;
#endif

#if MOBILE_USESUN
#if MOBILE_USEDOF
	half Sun0 = max(0.0, A0-1.0);
	half Sun1 = max(0.0, A1-1.0);
	half Sun2 = max(0.0, A2-1.0);
	half Sun3 = max(0.0, A3-1.0);
#else
	half Sun0 = A0;
	half Sun1 = A1;
	half Sun2 = A2;
	half Sun3 = A3;
#endif

	// Take average of sun intensity and adjust by bloom threshold.
	Amount *= 0.25;
	OutSunShaftAndDof += (Sun0 * Amount) + (Sun1 * Amount) + (Sun2 * Amount) + (Sun3 * Amount);
#endif

#if MOBILE_USEEYEADAPTATION
	const float Intensity = CalculateEyeAdaptationLuminance(AverageColor * View.OneOverPreExposure);
	const float LogIntensity = clamp(log2(Intensity), -10.0f, 20.0f);
	// Store log intensity in the alpha channel: scale to 0,1 range.
	OutEyeAdaptation = EyeAdaptation_HistogramScale * LogIntensity + EyeAdaptation_HistogramBias;
#endif
}






// 
// Bloom Downsample  
//

float BloomDownScale;

void BloomDownVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoord : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutTexCoord);
}


float4 InputViewportRect;     // xy = Min, zw = Size
float4 TextureSizeAndInvSize; // xy = TextureSize, zw = 1/TextureSize

void BloomDownPS_Mobile(
	float2 InUV : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
	const uint EyeIndex = GetEyeIndex(StereoInput);

	const float2 UVMin = (InputViewportRect.xy + 0.5f) * TextureSizeAndInvSize.zw;
	const float2 UVMax = (InputViewportRect.xy + InputViewportRect.zw - 0.5f) * TextureSizeAndInvSize.zw;

	const float Start = 2.0 / 14.0;
	const float Scale = BloomDownScale;

	// Circle offsets in UV space
	float2 Offset0  = Circle(Start, 14.0, 0.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset1  = Circle(Start, 14.0, 1.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset2  = Circle(Start, 14.0, 2.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset3  = Circle(Start, 14.0, 3.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset4  = Circle(Start, 14.0, 4.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset5  = Circle(Start, 14.0, 5.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset6  = Circle(Start, 14.0, 6.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset7  = Circle(Start, 14.0, 7.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset8  = Circle(Start, 14.0, 8.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset9  = Circle(Start, 14.0, 9.0)  * Scale * TextureSizeAndInvSize.zw;
	float2 Offset10 = Circle(Start, 14.0, 10.0) * Scale * TextureSizeAndInvSize.zw;
	float2 Offset11 = Circle(Start, 14.0, 11.0) * Scale * TextureSizeAndInvSize.zw;
	float2 Offset12 = Circle(Start, 14.0, 12.0) * Scale * TextureSizeAndInvSize.zw;
	float2 Offset13 = Circle(Start, 14.0, 13.0) * Scale * TextureSizeAndInvSize.zw;

	half4 N0 = SampleBloomDown(clamp(InUV,            UVMin, UVMax), EyeIndex);
	half4 N1 = SampleBloomDown(clamp(InUV + Offset0,  UVMin, UVMax), EyeIndex);
	half4 N2 = SampleBloomDown(clamp(InUV + Offset1,  UVMin, UVMax), EyeIndex);
	half4 N3 = SampleBloomDown(clamp(InUV + Offset2,  UVMin, UVMax), EyeIndex);
	half4 N4 = SampleBloomDown(clamp(InUV + Offset3,  UVMin, UVMax), EyeIndex);
	half4 N5 = SampleBloomDown(clamp(InUV + Offset4,  UVMin, UVMax), EyeIndex);
	half4 N6 = SampleBloomDown(clamp(InUV + Offset5,  UVMin, UVMax), EyeIndex);
	half4 N7 = SampleBloomDown(clamp(InUV + Offset6,  UVMin, UVMax), EyeIndex);
	half4 N8 = SampleBloomDown(clamp(InUV + Offset7,  UVMin, UVMax), EyeIndex);
	half4 N9 = SampleBloomDown(clamp(InUV + Offset8,  UVMin, UVMax), EyeIndex);
	half4 N10 = SampleBloomDown(clamp(InUV + Offset9,  UVMin, UVMax), EyeIndex);
	half4 N11 = SampleBloomDown(clamp(InUV + Offset10, UVMin, UVMax), EyeIndex);
	half4 N12 = SampleBloomDown(clamp(InUV + Offset11, UVMin, UVMax), EyeIndex);
	half4 N13 = SampleBloomDown(clamp(InUV + Offset12, UVMin, UVMax), EyeIndex);
	half4 N14 = SampleBloomDown(clamp(InUV + Offset13, UVMin, UVMax), EyeIndex);

	float W = 1.0 / 15.0;
	OutColor.rgb =
		(N0 * W) + (N1 * W) + (N2 * W) + (N3 * W) + 
		(N4 * W) + (N5 * W) + (N6 * W) + (N7 * W) + 
		(N8 * W) + (N9 * W) + (N10 * W) + (N11 * W) + 
		(N12 * W) + (N13 * W) + (N14 * W);
	OutColor.a = 0;
}


// 
// Bloom Upsample
//

float2 BloomUpScales;

void BloomUpVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoord : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutTexCoord);
}


float4 BloomTintA;
float4 BloomTintB;

float4 ViewportARect;           // xy = Min, zw = Size)
float4 ViewportBRect;           // xy = Min, zw = Size)
float4 TextureASizeAndInvSize;  // xy = TextureSize, zw = 1/TextureSize
float4 TextureBSizeAndInvSize;  // xy = TextureSize, zw = 1/TextureSize

void BloomUpPS_Mobile(
	float2 InUV : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
	const uint EyeIndex = GetEyeIndex(StereoInput);

	const float2 ViewportAUVScale = ViewportARect.zw * TextureASizeAndInvSize.zw;
	const float2 ViewportBUVScale = ViewportBRect.zw * TextureBSizeAndInvSize.zw;

	const float2 BaseUV_A = ViewportARect.xy * TextureASizeAndInvSize.zw + InUV * ViewportAUVScale;
	const float2 BaseUV_B = ViewportBRect.xy * TextureBSizeAndInvSize.zw + InUV * ViewportBUVScale;

	const float2 UVMinA = ViewportARect.xy * TextureASizeAndInvSize.zw;
	const float2 UVMaxA = (ViewportARect.xy + ViewportARect.zw - 0.5f) * TextureASizeAndInvSize.zw;
	const float2 UVMinB = ViewportBRect.xy * TextureBSizeAndInvSize.zw;
	const float2 UVMaxB = (ViewportBRect.xy + ViewportBRect.zw - 0.5f) * TextureBSizeAndInvSize.zw;

	const float Start = 2.0 / 7.0;
	const float ScaleA = BloomUpScales.x;
	const float ScaleB = BloomUpScales.y;

	const float2 OffsetA0 = Circle(Start, 7.0, 0.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA1 = Circle(Start, 7.0, 1.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA2 = Circle(Start, 7.0, 2.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA3 = Circle(Start, 7.0, 3.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA4 = Circle(Start, 7.0, 4.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA5 = Circle(Start, 7.0, 5.0) * ScaleA * TextureASizeAndInvSize.zw;
	const float2 OffsetA6 = Circle(Start, 7.0, 6.0) * ScaleA * TextureASizeAndInvSize.zw;

	const float2 OffsetB0 = Circle(Start, 7.0, 0.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB1 = Circle(Start, 7.0, 1.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB2 = Circle(Start, 7.0, 2.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB3 = Circle(Start, 7.0, 3.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB4 = Circle(Start, 7.0, 4.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB5 = Circle(Start, 7.0, 5.0) * ScaleB * TextureBSizeAndInvSize.zw;
	const float2 OffsetB6 = Circle(Start, 7.0, 6.0) * ScaleB * TextureBSizeAndInvSize.zw;

	const half3 A0 = SampleBloomUpA(clamp(BaseUV_A + OffsetA0, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A1 = SampleBloomUpA(clamp(BaseUV_A + OffsetA1, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A2 = SampleBloomUpA(clamp(BaseUV_A + OffsetA2, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A3 = SampleBloomUpA(clamp(BaseUV_A + OffsetA3, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A4 = SampleBloomUpA(clamp(BaseUV_A + OffsetA4, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A5 = SampleBloomUpA(clamp(BaseUV_A + OffsetA5, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A6 = SampleBloomUpA(clamp(BaseUV_A + OffsetA6, UVMinA, UVMaxA), EyeIndex).rgb;
	const half3 A7 = SampleBloomUpA(clamp(BaseUV_A,            UVMinA, UVMaxA), EyeIndex).rgb;

	const half3 B0 = SampleBloomUpB(clamp(BaseUV_B,            UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B1 = SampleBloomUpB(clamp(BaseUV_B + OffsetB0, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B2 = SampleBloomUpB(clamp(BaseUV_B + OffsetB1, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B3 = SampleBloomUpB(clamp(BaseUV_B + OffsetB2, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B4 = SampleBloomUpB(clamp(BaseUV_B + OffsetB3, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B5 = SampleBloomUpB(clamp(BaseUV_B + OffsetB4, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B6 = SampleBloomUpB(clamp(BaseUV_B + OffsetB5, UVMinB, UVMaxB), EyeIndex).rgb;
	const half3 B7 = SampleBloomUpB(clamp(BaseUV_B + OffsetB6, UVMinB, UVMaxB), EyeIndex).rgb;

	// A is the same size source.
	const half3 WA = BloomTintA.rgb;
	// B is the upsampled source.
	const half3 WB = BloomTintB.rgb;

	OutColor.rgb =
		A0 * WA + A1 * WA + A2 * WA + A3 * WA + A4 * WA + A5 * WA + A6 * WA + A7 * WA +
		B0 * WB + B1 * WB + B2 * WB + B3 * WB + B4 * WB + B5 * WB + B6 * WB + B7 * WB;
	OutColor.a = 0;
}


// 
// Near Setup - Generate near diolation for DOF.
//

void DofNearVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords2 : TEXCOORD0,
	out float4 OutTexCoords4[4] : TEXCOORD1,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, InTexCoord);

	OutTexCoords2 = InTexCoord;
	OutTexCoords4[0].xy = InTexCoord + BufferSizeAndInvSize.zw * float2(-0.5,-1.0);
	OutTexCoords4[0].zw = InTexCoord + BufferSizeAndInvSize.zw * float2( 1.0,-0.5);
	OutTexCoords4[1].xy = InTexCoord + BufferSizeAndInvSize.zw * float2( 0.5, 1.0);
	OutTexCoords4[1].zw = InTexCoord + BufferSizeAndInvSize.zw * float2(-1.0, 0.5);
	OutTexCoords4[2].xy = InTexCoord + BufferSizeAndInvSize.zw * float2( 0.5,-1.0);
	OutTexCoords4[2].zw = InTexCoord + BufferSizeAndInvSize.zw * float2( 1.0, 0.5);
	OutTexCoords4[3].xy = InTexCoord + BufferSizeAndInvSize.zw * float2(-0.5, 1.0);
	OutTexCoords4[3].zw = InTexCoord + BufferSizeAndInvSize.zw * float2(-1.0,-0.5);
}

void DofNearPS_Mobile(
	float2 InUVs2 : TEXCOORD0,
	float4 InUVs[4] : TEXCOORD1,
	in FStereoPSInput StereoInput,
	out HALF_TYPE OutColor : SV_Target0
	)
{

	half N0 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs2).r;
	half N1 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[0].xy).r;
	half N2 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[0].zw).r;
	half N3 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[1].xy).r;
	half N4 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[1].zw).r;
	half N5 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[2].xy).r;
	half N6 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[2].zw).r;
	half N7 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[3].xy).r;
	half N8 = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[3].zw).r;

	// Remove sunshaft intensity component and reverse.
	#if MOBILE_USESUN
		N0 = saturate(1.0 - N0);
		N1 = saturate(1.0 - N1);
		N2 = saturate(1.0 - N2);
		N3 = saturate(1.0 - N3);
		N4 = saturate(1.0 - N4);
		N5 = saturate(1.0 - N5);
		N6 = saturate(1.0 - N6);
		N7 = saturate(1.0 - N7);
		N8 = saturate(1.0 - N8);
	#else
		// If no sun-shafts then don't need the saturate.
		N0 = 1.0 - N0;
		N1 = 1.0 - N1;
		N2 = 1.0 - N2;
		N3 = 1.0 - N3;
		N4 = 1.0 - N4;
		N5 = 1.0 - N5;
		N6 = 1.0 - N6;
		N7 = 1.0 - N7;
		N8 = 1.0 - N8;
	#endif

	// The first sample is 1/4 the size as the rest of the samples.
	half Out = (N0 * 0.25 + N1 + N2 + N3 + N4 + N5 + N6 + N7 + N8) / 8.25;
	if(Out > 0.0) Out = sqrt(Out);
	OutColor = Out;
}


// 
// DOF Setup - Downsample to 1/4 area
//



void DofDownVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords[5] : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, InTexCoord);

	// Near position fixed to use UV based out output position.
	OutTexCoords[0] = OutPosition.xy * float2(0.5,-0.5) + 0.5;
	// Other source UVs based on possible non-full texture.
	OutTexCoords[1] = InTexCoord + BufferSizeAndInvSize.zw * float2(-0.5, -0.5);
	OutTexCoords[2] = InTexCoord + BufferSizeAndInvSize.zw * float2( 0.5, -0.5);
	OutTexCoords[3] = InTexCoord + BufferSizeAndInvSize.zw * float2(-0.5,  0.5);
	OutTexCoords[4] = InTexCoord + BufferSizeAndInvSize.zw * float2( 0.5,  0.5);
}

void DofDownPS_Mobile(
	float2 InUVs[5] : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
	// This shader needs float precision to work.

	// Fetch near diolation and scale to (0 to 16384.0) range.
	float N = DofNearTexture.Sample(DofNearSampler, InUVs[0]).r * 16384.0;

	const uint EyeIndex = GetEyeIndex(StereoInput);

	float4 A = SampleSceneColor(InUVs[1], EyeIndex);
	float4 B = SampleSceneColor(InUVs[2], EyeIndex);
	float4 C = SampleSceneColor(InUVs[3], EyeIndex);
	float4 D = SampleSceneColor(InUVs[4], EyeIndex);

	A.a = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[1]).r;
	B.a = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[2]).r;
	C.a = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[3]).r;
	D.a = SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[4]).r;

	#if MOBILE_USESUN
		// The {0.0 to 1.0} range is focus.
		// The {1.0 to 65504.0} range is light shaft source intensity (always at fully out of focus).
		// Must clamp back to {0.0 to 1.0} range.
		A.a = min(1.0, A.a);
		B.a = min(1.0, B.a);
		C.a = min(1.0, C.a);
		D.a = min(1.0, D.a);
	#endif

	// To support near DOF the {0.0 to 1.0} maps to {-16384.0 to 16384.0}.
	A.a = A.a * (2.0 * 16384.0) - 16384.0;
	B.a = B.a * (2.0 * 16384.0) - 16384.0;
	C.a = C.a * (2.0 * 16384.0) - 16384.0;
	D.a = D.a * (2.0 * 16384.0) - 16384.0;

	// Make sure there are no zeros.
	// Alpha ends up as circle of confusion size.
	// Near diolation factor applied here.
	// The 1/8 factor is to workaround mobile hardware lack of precision.
	A.a = max(N, abs(A.a) + 1.0/8.0);
	B.a = max(N, abs(B.a) + 1.0/8.0);
	C.a = max(N, abs(C.a) + 1.0/8.0);
	D.a = max(N, abs(D.a) + 1.0/8.0);

	// Mix weighted by circle of confusion.
	// This tends to erode the effect of more infocus samples (removes bleeding artifacts).
	OutColor = ((A * A.a) + (B * B.a) + (C * C.a) + (D * D.a)) * rcp(A.a + B.a + C.a + D.a);

	// Clamp rgb to prevent overflow during scale.
	OutColor.rgb = min(OutColor.rgb, 65504.0/16384.125);

	OutColor.rgb *= OutColor.a;
}


//
// DOF Blur
//

// DOF BOKEH SAMPLING PATTERN
// --------------------
// # = bilinear tap
// * = the single point tap to get the current pixel
//
//       1 1    
//   4 4 1 * 2 2
//   4 4 3 3 2 2 
//       3 3
//
// This pattern is very important.
// All bilinear taps are not always exactly in the middle of 4 texels.
// It is an asymetric pattern (minimize overlap, allow for different radii).
#define DOF_1 half2(-0.500, 0.50)
#define DOF_2 half2( 0.75,-0.50)
#define DOF_3 half2(-0.500,-1.25)
#define DOF_4 half2(-1.75,-0.50)



// This will compute a constant half2 from a constant half2.
// This computes the soft blend factor for intersection test
// (does circle of confusion intersect pixel center).
// Large feather here to make transitions smooth with a few samples.
half2 DofIntersectionScaleBias(half2 Offset) 
{
	// Working in distance squared.
	// Normalize by maximum distance 
	half RcpMaxDst = rcp(sqrt(dot(DOF_4, DOF_4)));
	half Dst0 = sqrt(dot(DOF_1, DOF_1));
	half Dst1 = sqrt(dot(Offset, Offset));
	Dst0 = max(Dst0, Dst1 - 0.25);
	Dst0 *= RcpMaxDst;
	Dst1 *= RcpMaxDst;
	half Scale = 1.0/(Dst1 - Dst0);
	half Bias = (-Dst0) * Scale;
	return half2(Scale, Bias);
}

half DofIntersect(half CocTap, half2 Offset)
{
	half2 ConstScaleBias = DofIntersectionScaleBias(Offset);
	// Undo the scale factor.
	ConstScaleBias.x *= 1.0/16384.0;
	return saturate(CocTap * ConstScaleBias.x + ConstScaleBias.y);
}

half DofWeight(half Coc) 
{
	half Dst0 = sqrt(dot(DOF_3, DOF_3)) / sqrt(dot(DOF_4, DOF_4));
	half Dst1 = sqrt(dot(DOF_4, DOF_4)) / sqrt(dot(DOF_4, DOF_4));
	half Scale = 1.0/(Dst1 - Dst0);
	half Bias = (-Dst0) * Scale;
	// Undo the 16384.0 scale factor in this constant.
	Scale *= 1.0/16384.0;
	// Scale and Bias should be compile time constants.
	return saturate(Coc * Scale + Bias);
}

void DofBlurVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords[5] : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, InTexCoord);

    OutTexCoords[0] = InTexCoord.xy;
    OutTexCoords[1] = InTexCoord.xy + float2(DOF_1) * BufferSizeAndInvSize.zw;
    OutTexCoords[2] = InTexCoord.xy + float2(DOF_2) * BufferSizeAndInvSize.zw;
    OutTexCoords[3] = InTexCoord.xy + float2(DOF_3) * BufferSizeAndInvSize.zw;
    OutTexCoords[4] = InTexCoord.xy + float2(DOF_4) * BufferSizeAndInvSize.zw;
}

void DofBlurPS_Mobile(
	float2 InUVs[5] : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
	// Near diolation size is copied into alpha for the tonemapper pass.
	OutColor.a = DofNearTexture.Sample(DofNearSampler, InUVs[0]).r;

	half4 C1 = DofDownTexture.Sample(DofDownSampler, InUVs[1]);
	half4 C2 = DofDownTexture.Sample(DofDownSampler, InUVs[2]);
	half4 C3 = DofDownTexture.Sample(DofDownSampler, InUVs[3]);
	half4 C4 = DofDownTexture.Sample(DofDownSampler, InUVs[4]);

	// Restore color (colors are weighted by CoC to help remove bleeding).
	C1.rgb *= rcp(C1.a);
	C2.rgb *= rcp(C2.a);
	C3.rgb *= rcp(C3.a);
	C4.rgb *= rcp(C4.a);

	// First bilinear tap always has 1.0 weight, the rest are weighted.
	half W1 = 1.0, W2, W3, W4;
	W2 = W3 = W4 = DofWeight(C1.a);
	
	// Remove contribution of taps who's circle of confusion does not intersect the pixel.
	W2 *= DofIntersect(C2.a, DOF_2);
	W3 *= DofIntersect(C3.a, DOF_3);
	W4 *= DofIntersect(C4.a, DOF_4);

	OutColor.rgb = ((C1.rgb * W1) + (C2.rgb * W2) + (C3.rgb * W3) + (C4.rgb * W4)) * rcp(W1 + W2 + W3 + W4);
}


// Integrate DOF

void IntegrateDOFVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float4 OutTexCoords : TEXCOORD0,
	out float2 OutFineDofGrain : TEXCOORD1,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutTexCoords.xy);

	// Fine adjustment is inside the possible non-full viewport in the full resolution texture.
	OutFineDofGrain.xy = OutTexCoords.xy + BufferSizeAndInvSize.zw * float2(-0.5, 0.5);
	// Want grain and a second UV based on the knowledge that the source texture has a full viewport.
	float2 FullViewUV = OutPosition.xy * float2(0.5, -0.5) + 0.5;
	// For DOF attempt to undo sampling bias for the first transition region.
	// This is better for the fine transition, breaks down for the larger bokeh.
	// This is the best compromise for mobile using 4 bilinear taps only.
	OutTexCoords.zw = FullViewUV.xy + DofBlurSizeAndInvSize.zw * float2(0.25, -0.5);
}

void IntegrateDOFPS_Mobile(
	in float4 TexCoords : TEXCOORD0,
	in float2 FineDofGrain : TEXCOORD1,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
)
{
	const uint EyeIndex = GetEyeIndex(StereoInput);
	
	half4 SceneColor = SampleSceneColor(TexCoords.xy, EyeIndex);
	half4 DofFine = SampleSceneColor(FineDofGrain.xy, EyeIndex);

	half4 Dof = Texture2DSample(DofBlurTexture, DofBlurSampler, TexCoords.zw);
	half DofCoc = Texture2DSample(SunShaftAndDofTexture, SunShaftAndDofSampler, TexCoords.xy).r;
	// Convert alpha back into circle of confusion.
	OutColor.a = SceneColor.a;
	SceneColor.a = max(Dof.a, abs(DofCoc * 2.0 - 1.0));
	// Convert circle of confusion into blend factors.		
	half2 ScaleBias = CocBlendScaleBias(); // Constant.
	half DofAmount = saturate(SceneColor.a * ScaleBias.x + ScaleBias.y);
	half2 ScaleBias2 = CocBlendScaleBiasFine(); // Constant.
	half DofAmountFine = saturate(SceneColor.a * ScaleBias2.x + ScaleBias2.y);
	// Blend in fine DOF.
	OutColor.rgb = lerp(SceneColor.rgb, DofFine.rgb, DofAmountFine);
	// Blend in coarse DOF.
	OutColor.rgb = lerp(OutColor.rgb, Dof.rgb, DofAmount);
}

//
// First sun shaft blur and move sun intensity from alpha to single channel output.
//

half HighlightCompression(half Channel) 
{
	return Channel * rcp(1.0 + Channel);
}
	
half HighlightDecompression(half Channel) 
{
	return Channel * rcp(1.0 - Channel);
}


// Convert from [-1 to 1] to view rectangle in texture which is somewhere in [0 to 1].
float2 SunShaftPosToUV(float2 Pos)
{
//	return (Pos.xy * ScreenPosToPixel.xy + ScreenPosToPixel.zw + 0.5f) * PostprocessInput0Size.zw;
	return Pos.xy * float2(0.5,-0.5) + 0.5;
}

// Center of light shaft.
float2 LightShaftCenter;

// Position in {-1 to 1} space.
float2 SunPos() 
{
	return LightShaftCenter.xy;
}

float2 SunShaftRect(float2 InPosition, float amount) 
{
	float2 center = SunPos();
	return SunShaftPosToUV(lerp(center, InPosition, amount));
}

// Positions for sun shaft steps.
// The very tight first position makes direct light to eye bloom a little.
// Otherwise want even spacing.
#define SUN_P0 (31.0/32.0)
#define SUN_P1 (27.0/32.0)
#define SUN_P2 (23.0/32.0)
#define SUN_P3 (19.0/32.0)
#define SUN_P4 (15.0/32.0)
#define SUN_P5 (11.0/32.0)
#define SUN_P6 (7.0/32.0)
// SUN_P7 is fixed at zero.

#define SUN_M 1.0

void SunAlphaVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords[8] : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, InTexCoord);

    OutTexCoords[0] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P0 * SUN_M);
    OutTexCoords[1] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P1 * SUN_M);
    OutTexCoords[2] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P2 * SUN_M);
    OutTexCoords[3] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P3 * SUN_M);
    OutTexCoords[4] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P4 * SUN_M);
    OutTexCoords[5] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P5 * SUN_M);
    OutTexCoords[6] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P6 * SUN_M);
    OutTexCoords[7] = InTexCoord.xy;
}

#undef SUN_M

// Remove the +1 bias.
// This sets negatives to zero because 0-1 is used for DOF.
half SunUnBias(half A)
{
	#if MOBILE_USEDOF
		return max(0.0, A - 1.0);
	#else
		return A;
	#endif
}

void SunAlphaPS_Mobile(
	float2 InUVs[8] : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF_TYPE OutColor : SV_Target0
	)
{
	OutColor = 
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[0]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[1]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[2]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[3]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[4]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[5]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[6]).r) * 0.125 +
		SunUnBias(SunShaftAndDofTexture.Sample(SunShaftAndDofSampler, InUVs[7]).r) * 0.125;
	OutColor = HighlightCompression(OutColor);
}




//
// Second sun shaft blur.
//

#define SUN_M 0.5

void SunBlurVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoords[8] : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, InTexCoord);

    OutTexCoords[0] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P0 * SUN_M);
    OutTexCoords[1] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P1 * SUN_M);
    OutTexCoords[2] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P2 * SUN_M);
    OutTexCoords[3] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P3 * SUN_M);
    OutTexCoords[4] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P4 * SUN_M);
    OutTexCoords[5] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P5 * SUN_M);
    OutTexCoords[6] = SunShaftRect(OutPosition.xy, 1.0 - SUN_P6 * SUN_M);
    OutTexCoords[7] = InTexCoord.xy;
}

#undef SUN_M

void SunBlurPS_Mobile(
	float2 InUVs[8] : TEXCOORD0,
	in FStereoPSInput StereoInput,
	out HALF_TYPE OutColor : SV_Target0
	)
{
	OutColor = 
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[0]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[1]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[2]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[3]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[4]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[5]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[6]).r * 0.125 +
		SunAlphaTexture.Sample(SunAlphaSampler, InUVs[7]).r * 0.125;
}


//
// Third sun shaft blur, composite with bloom, vignette.
//

#define SUN_M 0.25

float4 BloomUpTextureSizeAndInvSize; 	//xy=Size,zw=1/TexSize


void SunMergeVS_Mobile(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out float2 OutTexCoord : TEXCOORD0,
	out float4 OutSunShaftUVs[4] : TEXCOORD1,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutTexCoord);

	OutSunShaftUVs[0].xy = SunShaftRect(OutPosition.xy, 1.0 - SUN_P0 * SUN_M);
	OutSunShaftUVs[0].zw = SunShaftRect(OutPosition.xy, 1.0 - SUN_P1 * SUN_M);
	OutSunShaftUVs[1].xy = SunShaftRect(OutPosition.xy, 1.0 - SUN_P2 * SUN_M);
	OutSunShaftUVs[1].zw = SunShaftRect(OutPosition.xy, 1.0 - SUN_P3 * SUN_M);
	OutSunShaftUVs[2].xy = SunShaftRect(OutPosition.xy, 1.0 - SUN_P4 * SUN_M);
	OutSunShaftUVs[2].zw = SunShaftRect(OutPosition.xy, 1.0 - SUN_P5 * SUN_M);
	OutSunShaftUVs[3].xy = SunShaftRect(OutPosition.xy, 1.0 - SUN_P6 * SUN_M);
	OutSunShaftUVs[3].zw = float2(0.0, 0.0);
}

#undef SUN_M

float4 SunColorVignetteIntensity;
float3 BloomColor;

Texture2D BloomDirtMaskTexture;
SamplerState BloomDirtMaskSampler;
float4 BloomDirtMaskTint;

float4 SunMergeBloomSetupViewportRect;
float4 SunMergeBloomSetupTextureSizeAndInvSize;
float4 SunMergeBloomUpViewportRect;
float4 SunMergeBloomUpTextureSizeAndInvSize;

void SunMergePS_Mobile(
	float2 InUV : TEXCOORD0,
	float4 InSunShaftUVs[4] : TEXCOORD1,
	in FStereoPSInput StereoInput,
	out HALF4_TYPE OutColor : SV_Target0
	)
{
	const uint EyeIndex = GetEyeIndex(StereoInput);

	// UV scaling
	const float2 BloomSetupUVScale = SunMergeBloomSetupViewportRect.zw * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 BloomUpUVScale = SunMergeBloomUpViewportRect.zw * SunMergeBloomUpTextureSizeAndInvSize.zw;
	const float2 BloomSetupUV = SunMergeBloomSetupViewportRect.xy * SunMergeBloomSetupTextureSizeAndInvSize.zw + InUV * BloomSetupUVScale;
	const float2 BloomUpUV = SunMergeBloomUpViewportRect.xy * SunMergeBloomUpTextureSizeAndInvSize.zw + InUV * BloomUpUVScale;

	// UV clamping
	const float2 BloomSetupUVMin = SunMergeBloomSetupViewportRect.xy * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 BloomSetupUVMax = (SunMergeBloomSetupViewportRect.xy + SunMergeBloomSetupViewportRect.zw - 0.5f) * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 BloomUpUVMin = SunMergeBloomUpViewportRect.xy * SunMergeBloomUpTextureSizeAndInvSize.zw;
	const float2 BloomUpUVMax = (SunMergeBloomUpViewportRect.xy + SunMergeBloomUpViewportRect.zw - 0.5f) * SunMergeBloomUpTextureSizeAndInvSize.zw;

	// Circle sampling in PS
	const float Start = 2.0 / 6.0;
	const float Scale = 0.66 / 2.0;

	const float2 Offset0 = Circle(Start, 6.0, 0.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 Offset1 = Circle(Start, 6.0, 1.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 Offset2 = Circle(Start, 6.0, 2.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 Offset3 = Circle(Start, 6.0, 3.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 Offset4 = Circle(Start, 6.0, 4.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;
	const float2 Offset5 = Circle(Start, 6.0, 5.0) * Scale * SunMergeBloomSetupTextureSizeAndInvSize.zw;

	#if MOBILE_USEBLOOM

		const float Scale1 = 1.0 / 7.0;
		const float Scale2 = 1.0 / 7.0;

		const half3 Bloom2 = (
			SampleBloomSetup(clamp(BloomSetupUV,           BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale1 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset0, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset1, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset2, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset3, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset4, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2 +
			SampleBloomSetup(clamp(BloomSetupUV + Offset5, BloomSetupUVMin, BloomSetupUVMax), EyeIndex).rgb * Scale2) * rcp(Scale1 * 1.0 + Scale2 * 6.0);

		OutColor.rgb = SampleBloomUp(clamp(BloomUpUV, BloomUpUVMin, BloomUpUVMax), EyeIndex);

		const half3 BloomDirtMaskColor = BloomDirtMaskTexture.Sample(BloomDirtMaskSampler, InUV).rgb * BloomDirtMaskTint.rgb; // DirtMask is full-screen, sample with screen UV (not BloomSetupUV)

		// Have 5 layers on mobile.
		half Scale3 = 1.0/5.0;	

		// scale existing color first
		OutColor.rgb *= Scale3;		

		// add scaled bloom separately to prevent overflow before scaling
		OutColor.rgb += Bloom2 * Scale3 * BloomColor + BloomDirtMaskColor * OutColor.rgb;

	#else
		OutColor.rgb = half3(0.0, 0.0, 0.0);
	#endif

	#if MOBILE_USESUN
		half Sun = 
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[0].xy).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[0].zw).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[1].xy).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[1].zw).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[2].xy).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[2].zw).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InSunShaftUVs[3].xy).r * 0.125 +
			SunBlurTexture.Sample(SunBlurSampler, InUV).r * 0.125; // SunBlur is full-size RT, sample with screen UV (not BloomSetupUV)
		Sun = HighlightDecompression(Sun);
		OutColor.rgb += SunColorVignetteIntensity.rgb * Sun;
	#endif

	OutColor.a = 1.0f;
}

#undef SUN_P0
#undef SUN_P1
#undef SUN_P2
#undef SUN_P3
#undef SUN_P4
#undef SUN_P5
#undef SUN_P6

// EyeAdaptation

StructuredBuffer<float4> EyeAdaptationBuffer;

static const float FLOAT_PRECISION = 1e+5;

static const float INV_FLOAT_PRECISION = 1e-5;

float2 InputTextureSizeInv;

#if CLEAR_UAV_UINT_COMPUTE_SHADER
uint NumEntries;

uint ClearValue;
RWBuffer<uint> UAV;

[numthreads(64, 1, 1)]
void ClearUAVUIntCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.x < NumEntries)
	{
		UAV[DispatchThreadId.x] = ClearValue;
	}
}
#endif

#if AVERAGE_LUMINANCE_COMPUTE_SHADER || HISTOGRAM_COMPUTE_SHADER

float4 SourceSizeAndInvSize;

#endif

#if AVERAGE_LUMINANCE_COMPUTE_SHADER

RWBuffer<uint> OutputUIntBuffer;

groupshared float2 SharedLuminance[THREADGROUP_SIZEX * THREADGROUP_SIZEY];

// Each thread group processes LoopX * LoopY texels of the input.
const static uint2 TileSize = uint2(LOOP_SIZEX, LOOP_SIZEY) * 2; // Multiply 2 because we use bilinear sampler

const static uint ThreadGroupSize = THREADGROUP_SIZEX * THREADGROUP_SIZEY;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void AverageLuminance_MainCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex : SV_GroupIndex)
{
	// Top left input texel for this group.
	uint2 LeftTop = DispatchThreadId.xy * TileSize;

	uint2 Tile, TexelPos;
	half2 BufferUV;
	float2 SampleUV;
	float Weight;
	float Value;

	float2 GroupLuminance = float2(0.0f, 0.0f);

	LOOP for (uint y = 0; y < LOOP_SIZEY; ++y)
	{
		LOOP for (uint x = 0; x < LOOP_SIZEX; ++x)
		{
			Tile = uint2(x, y) * 2; // Multiply 2 because we use bilinear sampler
			TexelPos = LeftTop + Tile;

			if (TexelPos.x < SourceSizeAndInvSize.x && TexelPos.y < SourceSizeAndInvSize.y)
			{
				BufferUV = (half2)TexelPos + half2(1.0f, 1.0f); // offset pixel pos by 1 to use bilinear filter
				SampleUV = (float2)TexelPos * InputTextureSizeInv; // don't sample past edge of drr texture
				BufferUV = BufferUV * SourceSizeAndInvSize.zw;

				Weight = max(AdaptationWeightTexture(BufferUV), 0.05f);

				Value = SampleInput(SampleUV, 0).x;

				{
					// apply EyeAdaptation_BlackHistogramBucketInfluence using similar logic as Histogram method
					float fBucket = saturate(Value) * (HISTOGRAM_SIZE - 1);

					// Find two discrete buckets that straddle the continuous histogram position.
					uint Bucket0 = (uint)(fBucket);
					if (Bucket0 == 0)
					{
						Weight *= EyeAdaptation_BlackHistogramBucketInfluence;
					}
				}

				GroupLuminance.x += Value * Weight;
				GroupLuminance.y += Weight;
			}
		}
	}

	SharedLuminance[GroupIndex] = GroupLuminance;

	GroupMemoryBarrierWithGroupSync();

	UNROLL for (uint cutoff = (ThreadGroupSize >> 1); cutoff > 0; cutoff >>= 1)
	{
		if (GroupIndex < cutoff)
		{
			SharedLuminance[GroupIndex] += SharedLuminance[GroupIndex + cutoff];
		}

		if (cutoff > 4) // https://www.anandtech.com/show/12834/arm-announces-the-mali-g76-scaling-up-bifrost/2 said 4 is the wavefront for bifrost mali gpu
		{
			GroupMemoryBarrierWithGroupSync();
		}
	}

	if (GroupIndex <= 1)
	{
		float NormalizeFactor = SourceSizeAndInvSize.z * SourceSizeAndInvSize.w * 4.0f; // multiply 4 because bilinear filter, so sample only the 1/4 resolustion

		uint2 LuminanceInt = SharedLuminance[0] * NormalizeFactor * FLOAT_PRECISION;

		InterlockedAdd(OutputUIntBuffer[GroupIndex], LuminanceInt[GroupIndex]);
	}
}

#endif

Buffer<uint> LogLuminanceWeightBuffer;

float4 BasicEyeAdaptation_Mobile()
{
	float LogLumSum = LogLuminanceWeightBuffer[0];
	float WeightSum = LogLuminanceWeightBuffer[1];

	// When no tile contributed weight, fall back to 0 (the bottom of the histogram range) so the next frame can recover by brightening the scene.
	float LogLumAve = LogLumSum / max(WeightSum, 0.0000000001f);
	// Correct for [0,1] scaling
	LogLumAve = (LogLumAve - EyeAdaptation_HistogramBias) / EyeAdaptation_HistogramScale;

	// Convert LogLuminanceAverage to Average Intensity
	const float AverageSceneLuminance = exp2(LogLumAve);

	const float EyeAdaptationEV100 = log2((AverageSceneLuminance / 0.18) / EyeAdaptation_LuminanceMax);
	const float LUT_U = EyeAdaptationEV100 * EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.x + EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.y;
	const float ExposureCompensationFromCurveLUT = Texture2DSampleLevel(EyeAdaptation_ExposureCompensationCurveLUT, EyeAdaptation_ExposureCompensationCurveSampler, float2(LUT_U, 0.5f), 0).x;

	const float MiddleGreyExposureCompensation = EyeAdaptation_ExposureCompensationSettings * EyeAdaptation_ExposureCompensationCurve * ExposureCompensationFromCurveLUT * EyeAdaptation_GreyMult;// we want the average luminance remapped to 0.18, not 1.0

	const float ClampedLumAve = clamp(AverageSceneLuminance, EyeAdaptation_MinAverageLuminance, EyeAdaptation_MaxAverageLuminance);
	
	// The Exposure Scale (and thus intensity) used in the previous frame
	const float ExposureScaleOld = EyeAdaptationBuffer[0].x;

	const float LuminanceAveOld = MiddleGreyExposureCompensation / (ExposureScaleOld != 0.0f ? ExposureScaleOld : 1.0f);

	// Time-based expoential blend of the intensity to allow the eye adaptation to ramp up over a few frames.
	const float SmoothedLuminance = ComputeEyeAdaptation(LuminanceAveOld, ClampedLumAve, EyeAdaptation_DeltaWorldTime);

	const float SmoothedExposureScale = 1.0f / max(0.0001f, SmoothedLuminance);

	const float TargetExposureScale = 1.0f / max(0.0001f, ClampedLumAve);

	float4 OutColor;

	// Output the number that will rescale the image intensity
	OutColor.x = MiddleGreyExposureCompensation * SmoothedExposureScale;

	// Output the target value
	OutColor.y = MiddleGreyExposureCompensation * TargetExposureScale;

	OutColor.z = AverageSceneLuminance;

	OutColor.w = MiddleGreyExposureCompensation / EyeAdaptation_GreyMult;

	return OutColor;
}

#if BASIC_EYEADAPTATION_COMPUTE_SHADER
RWStructuredBuffer<float4> OutputBuffer;

[numthreads(1, 1, 1)]
void BasicEyeAdaptationCS_Mobile()
{
	OutputBuffer[0] = BasicEyeAdaptation_Mobile();
	OutputBuffer[1] = float4(1.0f, 0.0f, 0.0f, 0.0f);
}

#endif

const static float InvHistogramSize = 1.0f / (float)HISTOGRAM_SIZE;

const static float InvHistogramSizeMinusOne = 1.0f / (float) (HISTOGRAM_SIZE - 1);


#if HISTOGRAM_COMPUTE_SHADER
// Output histogram buffer (UAV)
RWBuffer<uint> RWHistogramBuffer;

const static uint QUARTER_HISTOGRAM_SIZE = HISTOGRAM_SIZE / 4;

// Each thread group processes LoopX * LoopY texels of the input.
const static uint2 TileSize = uint2(LOOP_SIZEX, LOOP_SIZEY) * 2; // Multiply 2 because we use bilinear sampler

const static uint ThreadGroupSize = THREADGROUP_SIZEX * THREADGROUP_SIZEY;

// THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms of the size HISTOGRAM_SIZE
groupshared float4 SharedHistogram[(HISTOGRAM_SIZE / 4) * THREADGROUP_SIZEX * THREADGROUP_SIZEY];

void WriteToHistogramBuffer(uint HitogramIndex, float NormalizeFactor)
{
	uint4 LuminanceInt = SharedHistogram[(HitogramIndex / 4) * ThreadGroupSize] * NormalizeFactor * FLOAT_PRECISION;

	uint LuminanceIntIndex = HitogramIndex % 4;

	InterlockedAdd(RWHistogramBuffer[HitogramIndex], LuminanceInt[LuminanceIntIndex]);
}

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void Histogram_MainCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex: SV_GroupIndex)
{
	// todo: can be cleared more efficiently
	// clear all THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms
	UNROLL for (uint i = 0; i < QUARTER_HISTOGRAM_SIZE; ++i)
	{
		SharedHistogram[i * ThreadGroupSize + GroupIndex] = float4(0.0f, 0.0f, 0.0f, 0.0f);
	}

	// Top left input texel for this group.
	uint2 LeftTop = DispatchThreadId.xy * TileSize;

	uint HistogramSizeMinusOne = HISTOGRAM_SIZE - 1;
	uint2 Tile, TexelPos;
	float2 BufferUV;
	float2 SampleUV;
	float LogLuminance, ScreenWeight, fBucket, Weight1, Weight0;
	uint x, y, Bucket0, Bucket1;

	// Accumulate all pixels into THREADGROUP_SIZEX*THREADGROUP_SIZEY histograms
	LOOP for (y = 0; y < LOOP_SIZEY; ++y)
	{
		LOOP for (x = 0; x < LOOP_SIZEX; ++x)
		{
			Tile = uint2(x, y) * 2; // Multiply 2 because we use bilinear sampler
			TexelPos = LeftTop + Tile;

			if(TexelPos.x < SourceSizeAndInvSize.x && TexelPos.y < SourceSizeAndInvSize.y)
			{
				BufferUV = (float2)TexelPos + float2(1.0f, 1.0f); // offset pixel pos by 1 to use bilinear filter
				SampleUV = (float2)TexelPos * InputTextureSizeInv; // don't sample past edge of drr texture
				BufferUV = BufferUV * SourceSizeAndInvSize.zw;

				LogLuminance = SampleInput(SampleUV, 0).x;
				ScreenWeight = AdaptationWeightTexture(BufferUV);

				// Map the normalized histogram position into texels.
				fBucket = LogLuminance * HistogramSizeMinusOne;

				// Find two discrete buckets that straddle the continuous histogram position.
				Bucket0 = (uint)(fBucket);
				Bucket1 = Bucket0 + 1;

				Bucket0 = min(Bucket0, HistogramSizeMinusOne);
				Bucket1 = min(Bucket1, HistogramSizeMinusOne);

				// Weighted blend between the two buckets.
				Weight1 = frac(fBucket);
				Weight0 = 1.0f - Weight1;
			
				// When EyeAdaptation_BlackHistogramBucketInfluence=.0, we will ignore the last bucket. The main use
				// case is so the black background pixels in the editor have no effect. But if we have cases where
				// pixel values can actually be black, we want to set EyeAdaptation_LastHistogramBucketInfluence=1.0.
				// This value is controlled by the cvar "r.EyeAdaptation.BlackHistogramBucketInfluence"
				if (Bucket0 == 0)
				{
					Weight0 *= EyeAdaptation_BlackHistogramBucketInfluence;
				}

				// Accumulate the weight to the nearby history buckets.
#if IOS // The IOS A8 and lower devices seems don't support using a float4 array as a two dimension array, separate the operations to two steps.
				float4 Histogram0 = float4(0.0f, 0.0f, 0.0f, 0.0f);
				float4 Histogram1 = float4(0.0f, 0.0f, 0.0f, 0.0f);

				Histogram0[Bucket0 % 4] = Weight0 * ScreenWeight;
				Histogram1[Bucket1 % 4] = Weight1 * ScreenWeight;

				SharedHistogram[(Bucket0 / 4) * ThreadGroupSize + GroupIndex] += Histogram0;
				SharedHistogram[(Bucket1 / 4) * ThreadGroupSize + GroupIndex] += Histogram1;
#else
				SharedHistogram[(Bucket0 / 4) * ThreadGroupSize + GroupIndex][Bucket0 % 4] += Weight0 * ScreenWeight;
				SharedHistogram[(Bucket1 / 4) * ThreadGroupSize + GroupIndex][Bucket1 % 4] += Weight1 * ScreenWeight;
#endif
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

// Reduction.
//
// float4 SharedHistogram[] is laid out like this:
//   [       float4         ] [       float4         ] ... [        float4         ] [       float4        ] [      float4         ] ...
//   [ Histogram 0, bins 0-3] [ Histogram 1, bins 0-3] ... [ Histogram 63, bins 0-3] [Histogram 0, bins 4-7] [Histogram 1, bins 4-7] ...
//
// To reduce we use HISTOGRAM_SIZE/4 threads to accumulate, where thread 0 accumulates bins 0-3 from all histograms, thread 1 bins 4-7, etc.
	if (GroupIndex < QUARTER_HISTOGRAM_SIZE)
	{
		float4 Sum = float4(0.0f, 0.0f, 0.0f, 0.0f);
		UNROLL for (uint i = 0; i < ThreadGroupSize; ++i)
		{

			// Accumulate bins from histogram i
			Sum += SharedHistogram[GroupIndex * ThreadGroupSize + i];
		}
		SharedHistogram[GroupIndex * ThreadGroupSize] = Sum;
	}

	GroupMemoryBarrierWithGroupSync();



	if (GroupIndex < HISTOGRAM_SIZE)
	{
		float NormalizeFactor = SourceSizeAndInvSize.z * SourceSizeAndInvSize.w * 4.0f; // multiply 4 because bilinear filter, so sample only the 1/4 resolustion

		WriteToHistogramBuffer(GroupIndex, NormalizeFactor);

#if LOW_SHARED_COMPUTE_MEMORY // Need to write two texels for each thread on LowSharedComputeMemory devices since there are only 32 threads in each tile.
		WriteToHistogramBuffer(GroupIndex * 2 + 1, NormalizeFactor);
#endif
	}
}

#endif

#if HISTOGRAM_EYEADAPTATION_COMPUTE_SHADER
Buffer<uint> HistogramBuffer;

// @param MinFractionSum e.g. ComputeHistogramSum() * 0.5f for 50% percentil
// @param MaxFractionSum e.g. ComputeHistogramSum() * 0.9f for 90% percentil
float ComputeAverageLuminanceWithoutOutlier_Mobile(float MinFractionSum, float MaxFractionSum)
{
	float2 SumWithoutOutliers = 0;

	UNROLL for (uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		float LocalValue = GetHistogramBucket(HistogramBuffer, i);

		// remove outlier at lower end
		float Sub = min(LocalValue, MinFractionSum);
		LocalValue = LocalValue - Sub;
		MinFractionSum -= Sub;
		MaxFractionSum -= Sub;

		// remove outlier at upper end
		LocalValue = min(LocalValue, MaxFractionSum);
		MaxFractionSum -= LocalValue;

		float LogLuminanceAtBucket = ComputeLogLuminanceFromHistogramPosition(float(i) * InvHistogramSizeMinusOne);

		SumWithoutOutliers += float2(LogLuminanceAtBucket, 1) * LocalValue;
	}

	//return SumWithoutOutliers.x / max(0.0001f, SumWithoutOutliers.y);
	float AvgLogLuminance = SumWithoutOutliers.x / max(0.0001f, SumWithoutOutliers.y);

	return exp2(AvgLogLuminance);
}

float ComputeEyeAdaptationExposure_Mobile()
{
	const float HistogramSum = ComputeHistogramSum(HistogramBuffer);
	const float AverageSceneLuminance = ComputeAverageLuminanceWithoutOutlier_Mobile(HistogramSum * EyeAdaptation_ExposureLowPercent, HistogramSum * EyeAdaptation_ExposureHighPercent);
	const float LumAve = AverageSceneLuminance;
	
	return LumAve;
}

RWStructuredBuffer<float4> OutputBuffer;

[numthreads(1, 1, 1)]
void HistogramEyeAdaptationCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float4 OutColor = 0;

	const float AverageSceneLuminance = ComputeEyeAdaptationExposure_Mobile();

	const float TargetAverageLuminance = clamp(AverageSceneLuminance, EyeAdaptation_MinAverageLuminance, EyeAdaptation_MaxAverageLuminance);
	
	const float InvGreyMult = 5.5555555556f; // 1.0f/0.18f

	// White point luminance is target luminance divided by 0.18 (18% grey).
	const float TargetExposure = TargetAverageLuminance * InvGreyMult;

	const float OldExposureScale = EyeAdaptationBuffer[0].x;

	const float EyeAdaptationEV100 = log2((AverageSceneLuminance / 0.18) / EyeAdaptation_LuminanceMax);
	const float LUT_U = EyeAdaptationEV100 * EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.x + EyeAdaptation_EV100ToExposureCompensationCurveLUTScaleBias.y;
	const float ExposureCompensationFromCurveLUT = Texture2DSampleLevel(EyeAdaptation_ExposureCompensationCurveLUT, EyeAdaptation_ExposureCompensationCurveSampler, float2(LUT_U, 0.5f), 0).x;

	const float MiddleGreyExposureCompensation = EyeAdaptation_ExposureCompensationSettings * EyeAdaptation_ExposureCompensationCurve * ExposureCompensationFromCurveLUT; // we want the average luminance remapped to 0.18, not 1.0
	const float OldExposure = MiddleGreyExposureCompensation / (OldExposureScale != 0 ? OldExposureScale : 1.0f);

	// eye adaptation changes over time
	const float EstimatedExposure = ComputeEyeAdaptation(OldExposure, TargetExposure, EyeAdaptation_DeltaWorldTime);

	// maybe make this an option to avoid hard clamping when transitioning between different exposure volumes?
	const float SmoothedExposure = clamp(EstimatedExposure, EyeAdaptation_MinAverageLuminance * InvGreyMult, EyeAdaptation_MaxAverageLuminance * InvGreyMult);

	const float SmoothedExposureScale = 1.0f / max(0.0001f, SmoothedExposure);
	const float TargetExposureScale =   1.0f / max(0.0001f, TargetExposure);

	OutColor.x = MiddleGreyExposureCompensation * SmoothedExposureScale;
	OutColor.y = MiddleGreyExposureCompensation * TargetExposureScale;
	OutColor.z = AverageSceneLuminance;
	OutColor.w = MiddleGreyExposureCompensation;

	OutputBuffer[0] = OutColor;
	OutputBuffer[1] = float4(1.0f, 0.0f, 0.0f, 0.0f);
}
#endif
