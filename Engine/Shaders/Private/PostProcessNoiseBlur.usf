// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessNoiseBlur.usf: PostProcessing down sample.
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"

// x:Radius
float4 NoiseParams;

#if METHOD == 0
	// todo
#elif METHOD == 1
	// todo
#elif METHOD == 2
	// todo
#else
	error
#endif

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;

	float2 PixelPos = SvPosition.xy;
	float Rnd = PseudoRandom(PixelPos); 

	float2 Offset = PostprocessInput0Size.zw;

	const int Count = 16;

	OutColor = 0;

	for(int i = 0; i < Count; ++i)
	{
		// more samples in the center, can be adjusted
		float r = (i + 1) / (float)Count * NoiseParams.x;
		
		// constant is tweaked for 16 samples
		float Angle = i * 8.1f + Rnd * 3.14159265f * 2;

		// can be optmized
		float2 Delta = float2(sin(Angle), cos(Angle)) * r;

		OutColor += Texture2DSample(PostprocessInput1, PostprocessInput1Sampler, UV + Delta * Offset);
	}

	OutColor /= Count;
}

