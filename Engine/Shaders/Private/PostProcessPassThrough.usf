// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessPassThrough.usf: PostProcessing Pass Through
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"

// pixel shader
void MainPS(noperspective float4 InUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = Texture2DSample(PostprocessInput0, PostprocessInput0Sampler, InUV.xy);
}
