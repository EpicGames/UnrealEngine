// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "GammaCorrectionCommon.ush"
#include "ACES/ACESCommon.ush"
#include "LensDistortion.ush"


Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, MSAA_SAMPLE_COUNT> EditorPrimitivesDepth;
Texture2DMS<uint2, MSAA_SAMPLE_COUNT> EditorPrimitivesStencil;
#else
// Note: opengl compiler doesn't like Texture2D<float>
Texture2D EditorPrimitivesDepth;
Texture2D<uint2> EditorPrimitivesStencil;
#endif

Texture2D ColorTexture;
SamplerState ColorSampler;

Texture2D DepthTexture;
SamplerState DepthSampler;

Texture2D OverlayLookupTexture;
SamplerState OverlayLookupSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Depth)
FScreenTransform PassSvPositionToViewportUV;
FScreenTransform ViewportUVToColorUV;
FScreenTransform ViewportUVToDepthUV;

float4 OutlineColors[8];
int OutlineColorIndexBits;
float SelectionHighlightIntensity;
float BSPSelectionIntensity;
float UILuminanceAndIsSCRGB;
float SecondaryViewportOffset;

struct FPixelInfo
{
	float fDepth;

	int ObjectId;
	// if the current pixel is not occluded by some other object in front of it
	float fVisible;
	// hard version of fVisible
	bool bVisible;
};

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ 
float ReconstructDeviceZ(float3 DeviceZPlane, float2 PixelOffset)
{
	return dot(DeviceZPlane, float3(PixelOffset.xy, 1));
}

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ
FPixelInfo GetPixelInfo(int2 PixelPos, int SampleID, int OffsetX, int OffsetY, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	FPixelInfo Ret;

	PixelPos += int2(OffsetX, OffsetY);

	// more stable on the silhouette
	float2 ReconstructionOffset = 0.5f - View.TemporalAAParams.zw;

#if MSAA_SAMPLE_COUNT > 1
	float DeviceZ = EditorPrimitivesDepth.Load(PixelPos, SampleID).r;
	int Stencil = EditorPrimitivesStencil.Load(PixelPos, SampleID) STENCIL_COMPONENT_SWIZZLE;

#if !COMPILER_GLSL && !COMPILER_PSSL && !COMPILER_METAL && !COMPILER_HLSLCC
	// not yet supported on OpenGL, slightly better quality
	ReconstructionOffset += EditorPrimitivesDepth.GetSamplePosition(SampleID);
#endif

#else
	float DeviceZ = EditorPrimitivesDepth.Load(int3(PixelPos, 0)).r;
	int Stencil = EditorPrimitivesStencil.Load(int3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
#endif

	float SceneDeviceZ = ReconstructDeviceZ(DeviceZPlane, ReconstructionOffset);

	// clamp SceneDeviceZ in (DeviceZMinMax.x .. DeviceZMinMax.z)
	// this avoids flicking artifacts on the silhouette by limiting the depth reconstruction error
	SceneDeviceZ = max(SceneDeviceZ, DeviceZMinMax.x);
	SceneDeviceZ = min(SceneDeviceZ, DeviceZMinMax.y);
	
	Ret.fDepth = DeviceZ;

	// outline even between multiple selected objects (best usability)
	Ret.ObjectId = Stencil;

#if (COMPILER_GLSL == 1 && FEATURE_LEVEL < FEATURE_LEVEL_SM4) || (COMPILER_METAL && FEATURE_LEVEL < FEATURE_LEVEL_SM4)
	// Stencil read in opengl is not supported on older versions
	Ret.ObjectId = (Ret.fDepth != 0.0f) ? 2 : 0;
#endif

	// Soft Bias with DeviceZ for best quality (larger bias than usual because SceneDeviceZ is only approximated)
	const float DeviceDepthFade = 0.01f;

	// 2 to always bias over the current plane
	Ret.fVisible = saturate(2.0f - (SceneDeviceZ - DeviceZ) / DeviceDepthFade);

	Ret.bVisible = Ret.fVisible >= 0.5f;

	return Ret;
}

bool BoolXor(bool bA, bool bB)
{
	int a = (int)bA;
	int b = (int)bB;

	return (a ^ b) != 0;
}

// Computes min and max at once.
void MinMax(inout int2 Var, int Value)
{
	Var.x = min(Var.x, Value);
	Var.y = max(Var.y, Value);
}

float4 PerSample(int2 PixelPos, int SampleID, FPixelInfo CenterPixelInfo, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	float4 Color = 0;

	// [0]:center, [1..4]:borders
	FPixelInfo PixelInfo[5];
	PixelInfo[0] = CenterPixelInfo;

	// Diagonal cross is thicker than vertical/horizontal cross.
	PixelInfo[1] = GetPixelInfo(PixelPos, SampleID, SecondaryViewportOffset + 1,  1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[2] = GetPixelInfo(PixelPos, SampleID, SecondaryViewportOffset - 1, -1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[3] = GetPixelInfo(PixelPos, SampleID, SecondaryViewportOffset + 1, -1, DeviceZPlane, DeviceZMinMax);
	PixelInfo[4] = GetPixelInfo(PixelPos, SampleID, SecondaryViewportOffset - 1,  1, DeviceZPlane, DeviceZMinMax);

	// With (.x != .y) we can detect a border around and between each object.
	int2 BorderMinMax = int2(255,0);
	{
		MinMax(BorderMinMax, PixelInfo[1].ObjectId);
		MinMax(BorderMinMax, PixelInfo[2].ObjectId);
		MinMax(BorderMinMax, PixelInfo[3].ObjectId);
		MinMax(BorderMinMax, PixelInfo[4].ObjectId);
	}

	int2 VisibleBorderMinMax = int2(255,0);
	{
		FLATTEN	if (PixelInfo[1].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[1].ObjectId);
		FLATTEN	if (PixelInfo[2].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[2].ObjectId);
		FLATTEN	if (PixelInfo[3].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[3].ObjectId);
		FLATTEN	if (PixelInfo[4].bVisible) MinMax(VisibleBorderMinMax, PixelInfo[4].ObjectId);
	}
	
	int ClosestPixelIndex = 0;
	{
		FLATTEN if (PixelInfo[1].fDepth > PixelInfo[ClosestPixelIndex].fDepth) ClosestPixelIndex = 1;
		FLATTEN if (PixelInfo[2].fDepth > PixelInfo[ClosestPixelIndex].fDepth) ClosestPixelIndex = 2;
		FLATTEN if (PixelInfo[3].fDepth > PixelInfo[ClosestPixelIndex].fDepth) ClosestPixelIndex = 3;
		FLATTEN if (PixelInfo[4].fDepth > PixelInfo[ClosestPixelIndex].fDepth) ClosestPixelIndex = 4;
	}

	// this border around the object
	bool bVisibleBorder = VisibleBorderMinMax.y != 0;
	bool bBorder = BorderMinMax.x != BorderMinMax.y;
	bool bInnerBorder = BorderMinMax.y == 4;

	// moving diagonal lines
	const uint2 uHalfResPixelPos = uint2(PixelPos)>>1u;
	float PatternMask = ((uHalfResPixelPos.x + uHalfResPixelPos.y) % 2) * 0.6f;

	// the contants express the two opacity values we see when the primitive is hidden
	float LowContrastPatternMask = lerp(0.2, 1.0f, PatternMask);

	const int BitsAvailable = 8;
	const int ColorShiftDistance = (BitsAvailable - OutlineColorIndexBits);
	const int ColorMask = ((0xFFu >> ColorShiftDistance) << ColorShiftDistance);
	int SelectionColorId = (PixelInfo[ClosestPixelIndex].ObjectId & ColorMask) >> ColorShiftDistance;
	float3 SelectionColor = OutlineColors[SelectionColorId].rgb;
	
	FLATTEN if (bBorder)
	{
		FLATTEN if (bVisibleBorder)
		{
			// unoccluded border
			Color = float4(SelectionColor, 1);
		}
		else
		{
			// occluded border
			Color = lerp(float4(0, 0, 0, 0), float4(SelectionColor, 1), LowContrastPatternMask);
		}
	}

	FLATTEN if (bInnerBorder)
	{
		// even occluded object is filled
		// occluded object is rendered differently(flickers with TemporalAA)
		float VisibleMask = lerp(PatternMask, 1.0f, PixelInfo[0].fVisible);

		// darken occluded parts
		Color = lerp(Color, float4(SelectionColor * VisibleMask, 1), SelectionHighlightIntensity);
	}

	// highlight inner part of the object, test against far plane
	if (PixelInfo[0].fDepth != 0.0f)
	{
		float VisibleMask = lerp(PatternMask, 1.0f, PixelInfo[0].fVisible);
		float InnerHighlightAmount = PixelInfo[0].ObjectId == 1 ? BSPSelectionIntensity : SelectionHighlightIntensity;
		Color = lerp(Color, float4(SelectionColor, 1), VisibleMask * InnerHighlightAmount);
	}

	return Color;
}

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const float2 DistortedViewportUV = ApplyScreenTransform(SvPosition.xy, PassSvPositionToViewportUV);
	const float2 UndistortedViewportUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, DistortedViewportUV);

	const float2 DistortedColorUV = ApplyScreenTransform(DistortedViewportUV, ViewportUVToColorUV);
	const float2 ColorUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToColorUV);
	const float2 DepthUV = ApplyScreenTransform(UndistortedViewportUV, ViewportUVToDepthUV);

	const int2 ColorPixelPos = int2(ColorUV * Color_Extent);

	// Scout outwards from the center pixel to determine if any neighboring pixels are selected
	const int ScoutStep = 2;

#if MSAA_SAMPLE_COUNT > 1
	const int4 StencilScout = int4(
		EditorPrimitivesStencil.Load(ColorPixelPos + int2( ScoutStep,  0) + int2(SecondaryViewportOffset, 0), 0) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(ColorPixelPos + int2(-ScoutStep,  0) + int2(SecondaryViewportOffset, 0), 0) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(ColorPixelPos + int2( 0,  ScoutStep) + int2(SecondaryViewportOffset, 0), 0) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(ColorPixelPos + int2( 0, -ScoutStep) + int2(SecondaryViewportOffset, 0), 0) STENCIL_COMPONENT_SWIZZLE
	);
#else
	const int4 StencilScout = int4(
		EditorPrimitivesStencil.Load(int3(ColorPixelPos + int2( ScoutStep,  0) + int2(SecondaryViewportOffset, 0), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(ColorPixelPos + int2(-ScoutStep,  0) + int2(SecondaryViewportOffset, 0), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(ColorPixelPos + int2( 0,  ScoutStep) + int2(SecondaryViewportOffset, 0), 0)) STENCIL_COMPONENT_SWIZZLE,
		EditorPrimitivesStencil.Load(int3(ColorPixelPos + int2( 0, -ScoutStep) + int2(SecondaryViewportOffset, 0), 0)) STENCIL_COMPONENT_SWIZZLE
	);
#endif

	float Center = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV, 0).r;
	float Left   = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + float2(-1,  0) * Depth_ExtentInverse, 0).r;
	float Right  = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + float2( 1,  0) * Depth_ExtentInverse, 0).r;
	float Top    = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + float2( 0, -1) * Depth_ExtentInverse, 0).r;
	float Bottom = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + float2( 0,  1) * Depth_ExtentInverse, 0).r;

	// This allows to reconstruct depth with a small pixel offset without many texture lookups (4xMSAA * 5 neighbors -> 20 samples)
	// It's an approximation assuming the surface is a plane.
	float3 DeviceZPlane;
	DeviceZPlane.x = (Right - Left) / 2;
	DeviceZPlane.y = (Bottom - Top) / 2;
	DeviceZPlane.z = Center;

	float2 DeviceZMinMax;
	DeviceZMinMax.x = min(Center, min(min(Left, Right), min(Top, Bottom)));
	DeviceZMinMax.y = max(Center, max(max(Left, Right), max(Top, Bottom)));

	FPixelInfo CenterPixelInfo = GetPixelInfo(ColorPixelPos, 0, SecondaryViewportOffset, 0, DeviceZPlane, DeviceZMinMax);

	float4 OverlayColor = Texture2DSample(OverlayLookupTexture, OverlayLookupSampler, DistortedColorUV);
	OutColor = Texture2DSample(ColorTexture, ColorSampler, DistortedColorUV); 
	
	float4 SelectionEffect = 0;
	const int ScoutSum = dot(saturate(StencilScout), 1);

	// If this sum is zero, none of our neighbors are selected pixels and we can skip all the heavy processing.
	if (ScoutSum > 0)
	{
		float4 Sum = 0;
#if COMPILER_GLSL && FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	UNROLL
#endif
		int SampleID = 0;
		// Workaround for platform preview with MSAA & FXC
		#if !(COMPILER_FXC && MOBILE_EMULATION)
		[unroll(MSAA_SAMPLE_COUNT)]
		for(SampleID = 0; SampleID < MSAA_SAMPLE_COUNT; ++SampleID)
		#endif
		{
			Sum += PerSample(ColorPixelPos, SampleID, CenterPixelInfo, DeviceZPlane, DeviceZMinMax);
		}
		SelectionEffect = Sum / MSAA_SAMPLE_COUNT;
	}
	
	// Scene color has gamma applied. Need to remove and re-apply after linear operations.
	if (SelectionEffect.a > 0 || OverlayColor.a > 0)
	{
#if SELECTION_OUTLINE_HDR
		static const float ScRGBScaleFactor = 80.0f; // used to convert between ScRGB and nits
		SelectionEffect.rgb *= abs(UILuminanceAndIsSCRGB);
		OverlayColor.rgb *= abs(UILuminanceAndIsSCRGB);
		if (UILuminanceAndIsSCRGB > 0.0)
		{
			OutColor.rgb *= ScRGBScaleFactor;
		}
		else
		{
			const float3x3 Rec2020_2_sRGB = mul(XYZ_2_sRGB_MAT, Rec2020_2_XYZ_MAT);
			OutColor.rgb = ST2084ToLinear(OutColor.rgb);
			OutColor.rgb = mul(Rec2020_2_sRGB, OutColor.rgb);
		}
#else
		OutColor.rgb = sRGBToLinear(OutColor.rgb);
#endif
		const uint2 uHalfResPixelPos = uint2(ColorPixelPos)>>1;
		float PatternMask = ((uHalfResPixelPos.x + uHalfResPixelPos.y) % 2) * 0.6f;
		float VisibleMask = lerp(PatternMask, 1.0f, CenterPixelInfo.fVisible);
		OutColor.rgb = lerp(OutColor.rgb, OverlayColor.rgb, OverlayColor.a * VisibleMask);
		OutColor.rgb = lerp(OutColor.rgb, SelectionEffect.rgb, SelectionEffect.a);

#if SELECTION_OUTLINE_HDR
		if (UILuminanceAndIsSCRGB > 0.0)
		{
			OutColor.rgb /= ScRGBScaleFactor;
		}
		else
		{
			const float3x3 sRGB_2_Rec2020 = mul(XYZ_2_Rec2020_MAT, sRGB_2_XYZ_MAT);
			OutColor.xyz = mul(sRGB_2_Rec2020, OutColor.xyz);
			OutColor.rgb = LinearToST2084(OutColor.rgb);
		}
#else
		OutColor.rgb = LinearToSrgbBranchless(OutColor.rgb);
#endif
	}
}
