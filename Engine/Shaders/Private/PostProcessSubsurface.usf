// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessSubsurface.usf: Screenspace subsurface scattering shaders.
=============================================================================*/

#include "Common.ush"
#include "Substrate/Substrate.ush"
#include "PostProcessCommon.ush"
#include "DeferredShadingCommon.ush"
#include "PostProcessSubsurfaceCommon.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

// for VisualizeSSS
#include "MiniFontCommon.ush"

#include "ScreenPass.ush"

Texture2D SubsurfaceInput0_Texture;
Texture2D SubsurfaceInput1_Texture;
Texture2D SubsurfaceInput2_Texture;
Texture2D SubsurfaceInput3_Texture;

SamplerState SubsurfaceSampler0;
SamplerState SubsurfaceSampler1;
SamplerState SubsurfaceSampler2;
SamplerState SubsurfaceSampler3;

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedSubsurfaceSampler0 SubsurfaceSampler0
	#define SharedSubsurfaceSampler1 SubsurfaceSampler0
	#define SharedSubsurfaceSampler2 SubsurfaceSampler0
	#define SharedSubsurfaceSampler3 SubsurfaceSampler0

#else
	#define SharedSubsurfaceSampler0 SubsurfaceSampler0
	#define SharedSubsurfaceSampler1 SubsurfaceSampler1
	#define SharedSubsurfaceSampler2 SubsurfaceSampler2
	#define SharedSubsurfaceSampler3 SubsurfaceSampler3
#endif

SCREEN_PASS_TEXTURE_VIEWPORT(SubsurfaceInput0)
SCREEN_PASS_TEXTURE_VIEWPORT(SubsurfaceInput1)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

RWTexture2D<uint> ProfileIdTexture;

//SUBSURFACE_SETUP_PASS
#define SUBSURFACE_SETUP_PASS_MAIN        0
#define SUBSURFACE_SETUP_PASS_PASSTHROUGH 1

#define SUBSURFACE_PASS_ONE 0
#define SUBSURFACE_PASS_TWO 1

#define SUBSURFACE_DIRECTION_HORIZONTAL SUBSURFACE_PASS_ONE
#define SUBSURFACE_DIRECTION_VERTICAL SUBSURFACE_PASS_TWO

// Controls the quality (number of samples) of the blur kernel.
#define SUBSURFACE_QUALITY_LOW 0
#define SUBSURFACE_QUALITY_MEDIUM 1
#define SUBSURFACE_QUALITY_HIGH 2

// Full resolution recombine.
#define SUBSURFACE_RECOMBINE_MODE_FULLRES 0

// Half resolution recombine.
#define SUBSURFACE_RECOMBINE_MODE_HALFRES 1

// Just reconstruct the lighting (needed for scalability).
#define SUBSURFACE_RECOMBINE_MODE_NO_SCATTERING 2

// Controls the quality of lighting reconstruction.
#define SUBSURFACE_RECOMBINE_QUALITY_LOW 0
#define SUBSURFACE_RECOMBINE_QUALITY_HIGH 1

#ifndef SUBSURFACE_RECOMBINE_QUALITY
	#define SUBSURFACE_RECOMBINE_QUALITY SUBSURFACE_RECOMBINE_QUALITY_LOW
#endif

#if SUBSURFACE_RECOMBINE_QUALITY == SUBSURFACE_RECOMBINE_QUALITY_HALFRES
	#define SUBSURFACE_HALFRES 1
#endif

// Validate if the checkerboard neighbor pixel is subsurface in recombine pass
#ifndef SUBSURFACE_RECOMBINE
	#define SUBSURFACE_RECOMBINE 0
#endif

#ifndef SUBSURFACE_SETUP_PASS
    #define SUBSURFACE_SETUP_PASS 0
#endif

#ifndef CHECKERBOARD_NEIGHBOR_SSS_VALIDATION
	#define CHECKERBOARD_NEIGHBOR_SSS_VALIDATION SUBSURFACE_RECOMBINE
	#if CHECKERBOARD_NEIGHBOR_SSS_VALIDATION
uint CheckerboardNeighborSSSValidation;
	#endif
#endif

#ifndef SUBSURFACE_DEBUG_PASS
#define SUBSURFACE_DEBUG_PASS 0
#endif

//=============================================================================
// setup for "SeparableSSS.ush"
//=============================================================================

#if SUBSURFACE_QUALITY == SUBSURFACE_QUALITY_LOW
	#define	SSSS_N_KERNELWEIGHTCOUNT SSSS_KERNEL2_SIZE
	#define	SSSS_N_KERNELWEIGHTOFFSET SSSS_KERNEL2_OFFSET
#elif SUBSURFACE_QUALITY == SUBSURFACE_QUALITY_MEDIUM
	#define	SSSS_N_KERNELWEIGHTCOUNT SSSS_KERNEL1_SIZE
	#define	SSSS_N_KERNELWEIGHTOFFSET SSSS_KERNEL1_OFFSET
#else // SUBSURFACE_QUALITY == SUBSURFACE_QUALITY_HIGH
	#define	SSSS_N_KERNELWEIGHTCOUNT SSSS_KERNEL0_SIZE
	#define	SSSS_N_KERNELWEIGHTOFFSET SSSS_KERNEL0_OFFSET
#endif

// 0: faster
// 1: no color bleeding in z direction
#define SSSS_FOLLOW_SURFACE 1

float GetMaskFromDepthInAlpha(float Alpha)
{
	return Alpha > 0;
}

#if SUBSTRATE_GBUFFER_FORMAT==1
FSubstrateSubsurfaceData LoadSubstrateSSSData(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateLoadSubsurfaceData(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelPos);
}
FSubstrateSubsurfaceHeader LoadSubstrateSSSHeader(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelPos);
}
#endif

// can be optimized
float GetSubsurfaceStrength(float2 UV)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(UV);
	const float Mask = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSHeader) ? 1.0f : 0.0f;
	const float RadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);
	const float Mask = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID) ? 1.0f : 0.0f;
	const float RadiusScale = ScreenSpaceData.GBuffer.CustomData.a;
#endif
	return Mask * RadiusScale;
}

// @return .rgb is the weight for color channel, .a is the sample location
float4 GetSubsurfaceProfileKernel(uint SampleIndex, uint SubsurfaceProfileInt)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	if (SubsurfaceProfileInt == SSS_PROFILE_ID_PERPIXEL) return float4(1, 1, 1, 1);
#endif
	const float4 TableMax = float4(1, 1, 1, SUBSURFACE_KERNEL_SIZE);

	return View.SSProfilesTexture.Load(int3(SampleIndex, SubsurfaceProfileInt, 0)) * TableMax;
}

float3 GetSubsurfaceProfileNormal(float2 BufferUV)
{

	BufferUV = clamp(BufferUV, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax);
	FGBufferData LocalGBufferData = GetGBufferData(BufferUV);

	return LocalGBufferData.WorldNormal;
}

uint GetSubsurfaceProfileId(float2 BufferUV)
{
	BufferUV = clamp(BufferUV, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax);
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(BufferUV);
	return SubstrateSubSurfaceHeaderGetProfileId(SSSHeader);
#else
	FGBufferData LocalGBufferData = GetGBufferData(BufferUV);
	return ExtractSubsurfaceProfileInt(LocalGBufferData);
#endif
}

float3 GetSubsurfaceProfileBoundaryColorBleed(uint SubsurfaceProfileInt)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	if (SubsurfaceProfileInt == SSS_PROFILE_ID_PERPIXEL) return float3(1.f, 1.f, 1.f);
#endif
	return GetSubsurfaceProfileKernel(SSSS_BOUNDARY_COLOR_BLEED_OFFSET, SubsurfaceProfileInt).rgb;
}

float4 GetSceneColor(float2 BufferUV)
{
	BufferUV = clamp(BufferUV, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax);

	return Texture2DSampleLevel(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, BufferUV, 0);
}

#if SUBSTRATE_GBUFFER_FORMAT==1

// Calculate the Subsurface profile kernel on the fly using integration instead of LUT.
half4 GetSubsurfaceProfileKernel(int SampleIndex, float3 D, float ScatterRadiusInMM, float WorldUnitScale)
{
	// See notes in SubsurfaceProfile.cpp. Scatter radius is in mm instead of cm here.
	ScatterRadiusInMM = max(ScatterRadiusInMM,1.0f);
	
	const float Range = ((2 * SSSS_N_KERNELWEIGHTCOUNT - 1) > 20) ? 3.0f : 2.0f;

	const float r = Range * float(SampleIndex) / (SSSS_N_KERNELWEIGHTCOUNT - 1);
	const float r0 = Range * (float(SampleIndex) - 0.5f) / (SSSS_N_KERNELWEIGHTCOUNT - 1);
	const float r1 = Range * (float(SampleIndex) + 0.5f) / (SSSS_N_KERNELWEIGHTCOUNT - 1);

	const float x = r * r / (Range );
	const float x0 = r0 * r0 / (Range );
	const float x1 = r1 * r1 / (Range );
	const float3 MaxCDF = GetCDF3D(D, ScatterRadiusInMM * Range * (1.0f + 0.5f / (SSSS_N_KERNELWEIGHTCOUNT - 1)));

	half4 Kernel = (half4)0.0f;

	BRANCH
		if (SampleIndex == 0)
		{
			Kernel.rgb = GetCDF3D(D, ScatterRadiusInMM * x1) / MaxCDF;
			Kernel.a = 0.0f;
		}
		else
		{
			// Need to multiply by 0.5f, and modify MaxCDF to match as close as offline LUT. Here favors integration 
			// instead of discrete approximation.
			Kernel.rgb = (GetCDF3D(D, ScatterRadiusInMM * x1) - GetCDF3D(D, ScatterRadiusInMM * x0)) / MaxCDF;
			Kernel.a = 2.0 * x; // 2.0f is consistent to the scale in ComputeMirroredBSSSKernel of BurleyNormalizedSSS.cpp
		}

	// Scale to match offline encoding
	// const float TableMaxA = SUBSURFACE_KERNEL_SIZE;
	// const FLinearColor TableColorScale = FLinearColor(
	//		1.0f / TableMaxRGB,
	//		1.0f / TableMaxRGB,
	//		1.0f / TableMaxRGB,
	//		1.0f / TableMaxA);
	// C *= TableColorScale
	// C.W *= Data.ScatterRadius / SUBSURFACE_RADIUS_SCALE;
	// SUBSURFACE_KERNEL_SIZE and TableMaxA cancels out
	Kernel.a *= (ScatterRadiusInMM * BURLEY_MM_2_CM) * WorldUnitScale /** SUBSURFACE_KERNEL_SIZE*/ /(/*TableMaxA **/ SUBSURFACE_RADIUS_SCALE);

	return Kernel;
}
#endif

// BufferUV back to grid posistion of the input
uint2 ConvertUVPos2Input0GridPos(float2 BufferUV)
{
	return clamp(BufferUV, 0, 1) * SubsurfaceInput0_Extent - 0.5f;
}

uint ExtractSubsurfaceProfileIntWithInvalid(float2 UV,float Depth)
{
	uint ProfileID = SSS_PROFILE_ID_INVALID;

	// the invalid status be determined by the scene depth.
	if (Depth > 0)
	{
		uint2 PosOfSampledUV = ConvertUVPos2Input0GridPos(UV);
		ProfileID = ProfileIdTexture[PosOfSampledUV];
#if SUBSTRATE_GBUFFER_FORMAT == 1
		//When Depth > 0, the profile id is valid, SSS_PROFILE_ID_PERPIXEL is compressed as SSS_PROFILE_ID_INVALID due to Profile ID caching texture being R8.
		ProfileID = (ProfileID == SSS_PROFILE_ID_INVALID) ? SSS_PROFILE_ID_PERPIXEL : ProfileID; 
#endif
	}
	return ProfileID;
}

// from https://github.com/iryoku/separable-sss/tree/master/Demo
// Jorge Jimenez http://www.iryoku.com/
// http://www.iryoku.com/translucency/downloads/Real-Time-Realistic-Skin-Translucency.pdf
#include "SeparableSSS.ush"

//=============================================================================

bool InUnitBox(float2 UV)
{
	return UV.x >= 0 && UV.y >= 0 && UV.y < 1 && UV.y < 1;
}

// @return 0=don't blend in, 1:fully blend in
float ComputeFullResLerp(float ProfileRadiusScale, float PixelRadiusScale, float2 UVSceneColor, float2 FullResInputSizeInverse)
{
	float SSSScaleX = SubsurfaceParams.x;

	float scale = SSSScaleX / CalcSceneDepth(UVSceneColor);

	float HorizontalScaler = SUBSURFACE_RADIUS_SCALE;

	// Calculate the final step to fetch the surrounding pixels:
	float finalStep = scale * HorizontalScaler;

	finalStep *= ProfileRadiusScale;

	float PixelSizeRadius = finalStep / (FullResInputSizeInverse.x * 0.5f);

	// tweaked for skin, a more flat kernel might need a smaller value, around 2 seems reasonable because we do half res
	const float PixelSize = 4.0f;

	float Ret = saturate(PixelSizeRadius - PixelSize);

	// opacity allows to scale the radius - at some point we should fade in the full resolution, we don't have a masking other than that.
	Ret *= saturate(PixelRadiusScale * 10);

	// todo: Subsurface has some non scatter contribution - all that should come from the Full res
	return Ret;
}

uint MiniFontCharFromSubsurfaceProfile(uint SubsurfaceProfileInt)
{
	if (SubsurfaceProfileInt <= 9)
	{
		return _0_ + SubsurfaceProfileInt;
	}

	SubsurfaceProfileInt -= 10;

	if (SubsurfaceProfileInt <= _Z_ - _A_)
	{
		return _A_ + SubsurfaceProfileInt;
	}

	return _QUESTIONMARK_;
}

// visualization (doesn't have to be fast)
void VisualizePS(in noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;

	OutColor = Texture2DSample(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, UV);

	int2 PixelPos = (int2)SvPosition.xy;

	float2 ViewLocalUV = (PixelPos - SubsurfaceInput0_ViewportMin) * SubsurfaceInput0_ViewportSizeInverse;

	float2 IDAreaLocalUV = ViewLocalUV * 2 - 1.0f;

	if (InUnitBox(IDAreaLocalUV))
	{
		float2 UV = SubsurfaceInput0_ViewportMin * SubsurfaceInput0_ExtentInverse + IDAreaLocalUV * (SubsurfaceInput0_ViewportSize * SubsurfaceInput0_ExtentInverse);

	#if SUBSTRATE_GBUFFER_FORMAT==1
		const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(UV);
		const uint SubsurfaceProfileInt = SubstrateSubSurfaceHeaderGetProfileId(SSSHeader);
		const bool bIsProfile = SubstrateSubSurfaceHeaderGetIsProfile(SSSHeader);
		const float PixelRadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
	#else
		const FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);
		const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
		const bool bIsProfile = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
		const float PixelRadiusScale = ScreenSpaceData.GBuffer.CustomData.a;
	#endif
		const float ProfileRadiusScale = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + SSSS_N_KERNELWEIGHTCOUNT - 1, SubsurfaceProfileInt).a;

		OutColor = float4(0.5f, 0.5f, 0.5f, 0);

		BRANCH if (bIsProfile)
		{
			if (SubsurfaceProfileInt == 0)
			{
				// default (no Profile)
				OutColor = float4(0.8f, 0.7f, 0.6f, 0);
			}
			if (SubsurfaceProfileInt == 1)
			{
				OutColor = float4(1, 0, 0, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 2)
			{
				OutColor = float4(0, 1, 0, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 3)
			{
				OutColor = float4(0, 0, 1, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 4)
			{
				OutColor = float4(1, 0, 1, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 5)
			{
				OutColor = float4(0, 1, 1, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 6)
			{
				OutColor = float4(1, 1, 0, 0) * 0.5f;
			}
			if (SubsurfaceProfileInt == 100)
			{
				OutColor = float4(0, 0.2f, 0, 0);
			}
			if (SubsurfaceProfileInt == 255)
			{
				OutColor = float4(1, 1, 1, 0);
			}

			int2 LeftTop = (PixelPos / 8) * 8;
			PrintCharacter(PixelPos, OutColor.rgb, float3(1, 1, 1), LeftTop, MiniFontCharFromSubsurfaceProfile(SubsurfaceProfileInt));

			OutColor.rgb *= ComputeFullResLerp(ProfileRadiusScale, PixelRadiusScale, UV, SubsurfaceInput0_ExtentInverse);
		}
	}
}


struct SDiffuseAndSpecular
{
	float3 Diffuse;
	float3 Specular;
};

#if CHECKERBOARD_NEIGHBOR_SSS_VALIDATION
bool IsCheckerBoardNeighborSubsurface(float2 SceneUV, int2 PixelOffset)
{
	const float2 UV = SceneUV + PixelOffset * View.BufferSizeAndInvSize.zw;

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(UV);
	const bool bHasSSSProfile = SubstrateSubSurfaceHeaderGetIsValid(SSSData.Header);
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);
	const bool bHasSSSProfile = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
#endif

	return bHasSSSProfile;
}
#endif

// can be moved/shared
half3 LookupSceneColor(float2 SceneUV, int2 PixelOffset)
{
	// faster
	float3 SceneColor = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, SceneUV, 0, PixelOffset).rgb;

#if CHECKERBOARD_NEIGHBOR_SSS_VALIDATION
	// Fix border background color leaking into subsurface diffuse/specular.
	BRANCH
	if (CheckerboardNeighborSSSValidation != 0)
	{
		bool bIsSubsurface = IsCheckerBoardNeighborSubsurface(SceneUV, PixelOffset);
		SceneColor = lerp(0, SceneColor, bIsSubsurface);
	}
#endif
	return SceneColor;
}

half3 ZeroAwareAdd(half3 A, half3 B)
{
	half3 R = A + B;
#if CHECKERBOARD_NEIGHBOR_SSS_VALIDATION
	R += select(all(A == 0.0f), B, 0.0f);
	R += select(all(B == 0.0f), A, 0.0f);
#endif
	return R;
}
 
// @param UVSceneColor for the full res rendertarget (BufferSize) e.g. SceneColor or GBuffers
// @param ReconstructMethod 0/1/2/3 (should be a literal constant to allow compiler optimizations)
SDiffuseAndSpecular ReconstructLighting(float2 UVSceneColor, uint ReconstructMethod)
{
	SDiffuseAndSpecular Ret;

#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	{
		// In this mode, we only read isolated diffuse lighting.
		float3 CenterSample = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, UVSceneColor, 0).rgb;
		Ret.Diffuse = CenterSample;
		Ret.Specular = 0.0f;
	}
	// If SUBSURFACE_CHANNEL_MODE is 0, checkerboard is forced on
#elif SUBSURFACE_PROFILE_CHECKERBOARD || SUBSURFACE_CHANNEL_MODE == 0
	{
	  bool bChecker = CheckerFromSceneColorUV(UVSceneColor);

	  // todo: We could alternate the diagonal with TemporalAA or even only only 1 sample for low spec or 4 for high spec

	  float3 Quant0 = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, UVSceneColor, 0).rgb;

	  // todo: expose as scalability setting (can be evaluate best without TemporalAA)
	  // 0:fast but pattern can appear, 1:better, 2: even better, 3: best but expensive
	  float3 Quant1;

	  if(ReconstructMethod == 0)
	  {
		  // cheap, crappy
		  Quant1 = LookupSceneColor(UVSceneColor, int2(1, 0));
	  }
	  else if(ReconstructMethod == 1)
	  {
		  // acceptable but not perfect
		  Quant1 = 0.5f * ZeroAwareAdd(
			  LookupSceneColor(UVSceneColor, int2( 1, 0)),
			  LookupSceneColor(UVSceneColor, int2(-1, 0)));
	  }
	  else if(ReconstructMethod == 2)
	  {
		  // almost same as 1?
		  half3 A = LookupSceneColor(UVSceneColor, int2(  1,  0));
		  half3 B = LookupSceneColor(UVSceneColor, int2(  0,  1));
		  half3 C = LookupSceneColor(UVSceneColor, int2( -1,  0));
		  half3 D = LookupSceneColor(UVSceneColor, int2(  0, -1));

		  Quant1 = 0.25f * ZeroAwareAdd(
			                     ZeroAwareAdd(A, B),
			                     ZeroAwareAdd(C, D));
	  }
	  else if(ReconstructMethod == 3)
	  {
		  // very good 
		  half3 A = LookupSceneColor(UVSceneColor, int2( 1,  0));
		  half3 B = LookupSceneColor(UVSceneColor, int2(-1,  0));
		  half3 C = LookupSceneColor(UVSceneColor, int2( 0,  1));
		  half3 D = LookupSceneColor(UVSceneColor, int2( 0, -1));

		  // Luminance could be green channel only
		  float a = Luminance(A);
		  float b = Luminance(B);
		  float c = Luminance(C);
		  float d = Luminance(D);

		  float ab = abs(a - b);
		  float cd = abs(c - d);

		  // take the average in the direction that avoids dither pattern
		  Quant1 = 0.5f * lerp(ZeroAwareAdd(A, B), ZeroAwareAdd(C, D), ab > cd);
	  }

	  Ret.Diffuse = lerp(Quant1, Quant0, bChecker);
	  Ret.Specular = lerp(Quant0, Quant1, bChecker);
	}
#else // SUBSURFACE_PROFILE_CHECKERBOARD
	{
	  // If we're not doing checkerboard encoding, we just need to read a single pixel and decode (combined diffuse/spec in RGB)
       float4 CenterSample;

#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	  BRANCH
	  if (bUseSeparatedSubsurfaceDiffuse)
	  {
		  CenterSample.rgb = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, UVSceneColor, 0).rgb;
		  CenterSample.a = GetSeparatedSubsurfaceDiffuseLuminance(SharedSubsurfaceSampler0, UVSceneColor);
		  CenterSample = DecodeSeparatedSubsurfaceDiffuse(CenterSample);
	  }
	  else
#endif
	  {
		  CenterSample = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, UVSceneColor, 0);
	  }
	  
       float3 CombinedColor = CenterSample.rgb;
       float DiffuseLuminance = CenterSample.a;
	  float CombinedLuminance = Luminance(CombinedColor);
	  float DiffuseFactor = saturate(DiffuseLuminance / CombinedLuminance);
	  float SpecularFactor = 1.0f - DiffuseFactor;
  
	  Ret.Diffuse = CombinedColor * DiffuseFactor;
	  Ret.Specular = CombinedColor * SpecularFactor;
	}
#endif // !SUBSURFACE_PROFILE_CHECKERBOARD

	return Ret;
}

// @param UVSceneColor for the full res rendertarget (BufferSize) e.g. SceneColor or GBuffers
// @return .RGB Color that should be scattared, .A:1 for subsurface scattering material, 0 for not
float4 SetupSubsurfaceForOnePixel(float2 UVSceneColor)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(UVSceneColor);
	const bool bHasSSSDiffusion = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSHeader);
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UVSceneColor);
	const bool bHasSSSDiffusion = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
#endif

	float4 Ret = 0;
	BRANCH if (bHasSSSDiffusion)
	{
		// '1' is lower quality but that is acceptable here in regular rendering
		// When rendering resolution is very low, '3' is required to reduce checkerboard pattern.
		uint CheckerboardReconstructMethod = (SUBSURFACE_RECOMBINE_QUALITY == SUBSURFACE_RECOMBINE_QUALITY_HIGH) ? 3 : 1;
		SDiffuseAndSpecular DiffuseAndSpecular = ReconstructLighting(UVSceneColor, CheckerboardReconstructMethod);

		Ret.rgb = DiffuseAndSpecular.Diffuse;

		// it's a valid sample
		Ret.a = 1;
	}
	return Ret;
}

Buffer<uint> TileTypeCountBuffer;

bool ShouldGenerateMipmaps(uint TileType)
{
	bool bShouldGenerateMipmaps = false;

	if (SUBSURFACE_TILE_TYPE_AFIS == TileType)
	{
		uint TileCount = TileTypeCountBuffer[TileType];
		bShouldGenerateMipmaps = (TileCount >= SubsurfaceParams.w);
	}

	return bShouldGenerateMipmaps;
}

// Ref: https://graphics.pixar.com/library/ApproxBSSRDF/paper.pdf
#include "SubsurfaceBurleyNormalized.ush"

#if SUBSURFACE_BURLEY_COMPUTE
//-----------------------------------------------------------------------------------
// Indirect pass

// Indirect dispatch parameters
RWBuffer<uint> RWSeparableGroupBuffer;
RWBuffer<uint> RWBurleyGroupBuffer;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;
RWBuffer<uint> RWIndirectDrawArgsBuffer;
RWBuffer<uint> RWTileTypeCountBuffer;
Buffer<uint> GroupBuffer;

[numthreads(1, 1, 1)]
void InitValueBufferCS()
{
	// Clear the buffer counter for all types of tile.
	UNROLL
	for (uint i = 0; i < SUBSURFACE_TILE_TYPE_COUNT; ++i)
	{
		RWTileTypeCountBuffer[i] = 0;
	}
}

uint MinGenerateMipsTileCount;
uint Offset;
RWBuffer<uint> RWMipsConditionBuffer;

[numthreads(1, 1, 1)]
void FillMipsConditionBufferCS()
{
	uint TileCount = TileTypeCountBuffer[Offset];
	RWMipsConditionBuffer[Offset] = (TileCount >= MinGenerateMipsTileCount) ? 1 : 0;
}

[numthreads(8, 1, 1)]
void BuildIndirectDispatchArgsCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.y == 0 && DispatchThreadId.x < SUBSURFACE_TILE_TYPE_COUNT 
		&& DispatchThreadId.x != SUBSURFACE_TILE_TYPE_PASSTHROUGH
		&& DispatchThreadId.x != SUBSURFACE_TILE_TYPE_ALLNONPASSTHROUGH
		&& DispatchThreadId.x != SUBSURFACE_TILE_TYPE_ALL)
	{
		uint TileType = DispatchThreadId.x;
		const uint TileCount = TileTypeCountBuffer[TileType];
		
		// Indirect draw
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 0] = SubsurfaceUniformParameters.bRectPrimitive > 0 ? 4 : 6;  // VertexCountPerInstance
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 1] = TileCount;  // InstanceCount
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 2] = 0;  // StartVertexLocation
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 3] = 0;  // StartInstanceLocation

		// Indirect dispatch
		const uint IndirectDispatchSize = TileCount; // (TileCount + (THREAD_SIZE_1D*THREAD_SIZE_1D) - 1) / (THREAD_SIZE_1D*THREAD_SIZE_1D);
		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, TileType, IndirectDispatchSize, 1, 1);
	}
}

uint GetSubsurfaceType(uint ProfileId)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	// We don't have a bit to indicate which SSS implementation to use, so default to TYPE_BURLEY that uses AFIS. This is default behavior previously.
	if (ProfileId == SSS_PROFILE_ID_PERPIXEL) return TYPE_BURLEY;
#endif
	return GetSubsurfaceProfileUseBurley(ProfileId) ? TYPE_BURLEY : TYPE_SEPARABLE;
}

void AddSeparableGroup(uint2 GroupXY)
{
	//buffer counter is stored in the .Count in the first element.
	uint IndexToStore = 0;

	InterlockedAdd(RWTileTypeCountBuffer[SUBSURFACE_TILE_TYPE_SEPARABLE], 1, IndexToStore);

	if (IndexToStore >= SubsurfaceUniformParameters.MaxGroupCount)
	{
		return;
	}

	RWSeparableGroupBuffer[IndexToStore] = PackTileCoord12bits(GroupXY);
}

void AddBurleyGroup(uint2 GroupXY)
{
	//buffer counter is stored in the .Count in the first element.
	uint IndexToStore = 0;

	InterlockedAdd(RWTileTypeCountBuffer[SUBSURFACE_TILE_TYPE_AFIS], 1, IndexToStore);

	if (IndexToStore >= SubsurfaceUniformParameters.MaxGroupCount)
	{
		return;
	}

	RWBurleyGroupBuffer[IndexToStore] = PackTileCoord12bits(GroupXY);
}

void AddSubsurfaceComputeGroup(uint GI, uint Type, uint2 GroupXY)
{
#if COMPILER_SUPPORTS_WAVE_VOTE && COMPILER_SUPPORTS_WAVE_BIT_ORAND
	const uint SSSInViewInWave = WaveActiveBitOr(Type);
	const bool AnySSSInViewInWave =  (SSSInViewInWave & TYPE_IN_VIEWPORT) &&
		                             ((SSSInViewInWave & TYPE_SEPARABLE)||
									  (SSSInViewInWave & TYPE_BURLEY));
	if (WaveIsFirstLane() && AnySSSInViewInWave)
	{
		InterlockedOr(SubsurfaceTypeFlag, SSSInViewInWave);
	}
#else
	InterlockedOr(SubsurfaceTypeFlag, Type);
#endif

	GroupMemoryBarrierWithGroupSync();

	if (GI == 0 && ((SubsurfaceTypeFlag & TYPE_SEPARABLE) && (SubsurfaceTypeFlag & TYPE_IN_VIEWPORT)))
	{
		AddSeparableGroup(GroupXY);
	}

	if (GI == 0 && ((SubsurfaceTypeFlag & TYPE_BURLEY) && (SubsurfaceTypeFlag & TYPE_IN_VIEWPORT)))
	{
		AddBurleyGroup(GroupXY);
	}
}

// Get random index that has Mask[index] = 1 (s.t. Sum(Mask)>=1)
uint GetRandomIndex(uint2 Pos, uint4 Mask)
{
	uint4 PrefixSum = uint4(
		Mask[0],
		Mask[0]+Mask[1],
		Mask[0]+Mask[1]+Mask[2],
		Mask[0]+Mask[1]+Mask[2]+Mask[3]);

	uint RandomSelectedBit = (Rand3DPCG16(int3(Pos, View.FrameNumber)).x % PrefixSum[3]) + 1u;
	
	uint m0 = (RandomSelectedBit == PrefixSum[0]) & Mask[0];
	uint m1 = (RandomSelectedBit == PrefixSum[1]) & Mask[1];
	uint m2 = (RandomSelectedBit == PrefixSum[2]) & Mask[2];
	uint m3 = (RandomSelectedBit == PrefixSum[3]) & Mask[3];

	uint SelectedIndexWithValidMask = firstbitlow((m0 << 0u)|(m1 << 1u)|(m2 << 2u)|(m3 << 3u));
	return min(SelectedIndexWithValidMask,3u);
}

RWTexture2D<float4> SetupTexture;
RWTexture2D<float4> ProfileEdgeMask;

[numthreads(THREAD_SIZE_X*THREAD_SIZE_Y, 1, 1)]
void SetupIndirectCS(	
	uint3 IndirectG_ID : SV_GroupID,
	uint  GI : SV_GroupIndex)
{
	// initalize the group shared variable
	if (GI == 0)
	{
		SubsurfaceTypeFlag = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	uint2 G_ID = UnpackTileCoord12bits(GroupBuffer[IndirectG_ID.x]);
	uint2 DT_ID = G_ID * float2(THREAD_SIZE_X, THREAD_SIZE_Y) + float2(GI % THREAD_SIZE_X, GI / THREAD_SIZE_X);

	uint2 Pos = DT_ID.xy + Output_ViewportMin;
	float2 BufferUV = ConvertGridPos2UV(Pos);
	uint Type = 0;
	float4 OutColor = 0;
	float4 OutColor2 = 0;
#if SUBSURFACE_HALF_RES
	// order aligned with Gather() hardware implementation
	// RGB: color*A, A:weight 0 if no subsurface scattering

	float4 A = SetupSubsurfaceForOnePixel(min(BufferUV + float2(-0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax));
	float4 B = SetupSubsurfaceForOnePixel(min(BufferUV + float2(0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax));
	float4 C = SetupSubsurfaceForOnePixel(min(BufferUV + float2(0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax));
	float4 D = SetupSubsurfaceForOnePixel(min(BufferUV + float2(-0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax));

	float4 Sum = (A + B) + (C + D);

	// Each alpha value of A/B/C/D should be either 1.0f or 0.0, so a check of 0.5 should be safe
	bool bHasSubsurface = (Sum.a >= .50f);

	// Sum.a should either be one of {0,1,2,3,4}, add a max of 1 to avoid floating point specials. If Sum.a is 0, Sum.rgb should be 0 as well so it
	// shouldn't matter what Div is.
	float Div = 1.0f / max(Sum.a, 1.0f);

	OutColor.rgb = Sum.rgb * Div;

	float4 FourDepth = GatherSceneDepth(BufferUV, SubsurfaceInput0_ExtentInverse);

	// average all valid depth values to a single one
	float SingleDepth = dot(FourDepth, float4(A.a, B.a, C.a, D.a)) * Div;

	OutColor.a = SingleDepth;

	if (OutColor.a > 0)
	{
		bHasSubsurface = true;
	}
	//BufferUV += min(float2(-0.25, -0.25f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
#else // SUBSURFACE_HALF_RES
	
	bool bHasSubsurface = false;
	OutColor = SetupSubsurfaceForOnePixel(BufferUV);

	if (OutColor.a > 0)
	{
		bHasSubsurface = true;

		float SourceDepth = CalcSceneDepth(BufferUV);

		float Noise = InterleavedGradientNoise(float2(Pos.x,Pos.y), View.StateFrameIndexMod8);
		// Divide by 1024 because the mantissa is 10 bits.
		OutColor.a = SourceDepth + Noise * (SourceDepth / 1024.0f);

		//@TODO: sparkling
		//supress the edge color lighting information at edge.
		//with firefly elimination
	}
#endif // SUBSURFACE_HALF_RES

#if ENABLE_PROFILE_ID_CACHE
	uint ProfileId = GetSubsurfaceProfileId(BufferUV);
#endif

	// setup the Subsurface type
	uint SelectedProfile = 0;
	if (bHasSubsurface)
	{
#if SUBSURFACE_HALF_RES

	#if !SUBSURFACE_FORCE_SEPARABLE

        #if 1
		{
			float2 UVA = min(BufferUV + float2(-0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			float2 UVB = min(BufferUV + float2(0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			float2 UVC = min(BufferUV + float2(0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			float2 UVD = min(BufferUV + float2(-0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);

			BufferUV = lerp(BufferUV, UVA, A.a);
			BufferUV = lerp(BufferUV, UVB, B.a);
			BufferUV = lerp(BufferUV, UVC, C.a);
			BufferUV = lerp(BufferUV, UVD, D.a);
		}
		#else
		{
			// Gatter available and randomize.
			float4x2 UVs;
			UVs[0] = min(BufferUV + float2(-0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			UVs[1] = min(BufferUV + float2(0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			UVs[2] = min(BufferUV + float2(0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
			UVs[3] = min(BufferUV + float2(-0.5, -0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);

			uint UVIndex = GetRandomIndex(Pos, uint4(A.a, B.a, C.a, D.a));
			BufferUV = UVs[UVIndex];
		}
		#endif
		
		SelectedProfile = GetSubsurfaceProfileId(BufferUV);
		Type = GetSubsurfaceType(SelectedProfile);
#if ENABLE_PROFILE_ID_CACHE
		ProfileId = SelectedProfile;
#endif
	#else
		Type = TYPE_SEPARABLE;
	#endif
		
#else
#if ENABLE_PROFILE_ID_CACHE
		SelectedProfile = ProfileId;
#else
		SelectedProfile = GetSubsurfaceProfileId(BufferUV);
#endif
		
		Type = GetSubsurfaceType(SelectedProfile);
#endif
	}

	if (all(Pos < Output_ViewportMax))
	{
		Type |= TYPE_IN_VIEWPORT;
#if (SUBSURFACE_SETUP_PASS == SUBSURFACE_SETUP_PASS_PASSTHROUGH)
		//write to the final buffer, need mask to recombine.
		OutColor.a = GetMaskFromDepthInAlpha(OutColor.a);
#endif
		SetupTexture[Pos] = OutColor;

#if ENABLE_PROFILE_ID_CACHE
		ProfileIdTexture[Pos] = ProfileId;
#endif
	}

#if !(SUBSURFACE_FORCE_SEPARABLE || (SUBSURFACE_SETUP_PASS == SUBSURFACE_SETUP_PASS_PASSTHROUGH))
	AddSubsurfaceComputeGroup(GI, Type, G_ID.xy);
#endif
}

[numthreads(THREAD_SIZE_X*THREAD_SIZE_Y, 1, 1)]
void MainIndirectDispatchCS(
	uint3 IndirectG_ID : SV_GroupID,
	uint  GI : SV_GroupIndex
) 
{
	uint2 G_ID = UnpackTileCoord12bits(GroupBuffer[IndirectG_ID.x]);
	uint2 DT_ID = G_ID * float2(THREAD_SIZE_X, THREAD_SIZE_Y) + float2(GI % THREAD_SIZE_X, GI / THREAD_SIZE_X);


	//1. Call function based on which function is defined and in which phase
#if SUBSURFACE_TYPE == SSS_TYPE_BURLEY && !SUBSURFACE_FORCE_SEPARABLE

	BurleyComputeMain(DT_ID, G_ID, GI);

#elif SUBSURFACE_TYPE == SSS_TYPE_SSSS

	uint2 Pos = DT_ID.xy*SUBSURFACE_GROUP_SIZE / THREAD_SIZE_1D + Output_ViewportMin;
	float2 BufferUV = ConvertGridPos2UV(Pos);
	uint2  BufferPos = Pos;

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(BufferUV);
	const uint bUseBurley = SubstrateSubSurfaceHeaderGetIsProfile(SSSHeader) ? GetSubsurfaceProfileUseBurley(SubstrateSubSurfaceHeaderGetProfileId(SSSHeader)) : SubstrateSubSurfaceHeaderGetUseDiffusion(SSSHeader);
#else
	const FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
	const bool bUseBurley = GetSubsurfaceProfileUseBurley(SubsurfaceProfileInt);
#endif

	// We need to check Burley here because the previous pass might write burley samplings here unless we have SUBSURFACE_FORCE_SEPARABLE to overwrite the setting.
#if !SUBSURFACE_FORCE_SEPARABLE
	BRANCH if (bUseBurley)
	{
		return;
	}
#endif
	
	// pass one and pass two
#if SUBSURFACE_PASS == SUBSURFACE_PASS_ONE
	// horizontal
	float2 ViewportDirectionUV = float2(1, 0) * SUBSURFACE_RADIUS_SCALE;
#elif SUBSURFACE_PASS == SUBSURFACE_PASS_TWO
	// vertical
	float2 ViewportDirectionUV = float2(0, 1) * SUBSURFACE_RADIUS_SCALE * (SubsurfaceInput0_Extent.x * SubsurfaceInput0_ExtentInverse.y);
#endif

	ViewportDirectionUV *= (SubsurfaceInput0_ViewportSize.x * SubsurfaceInput0_ExtentInverse.x);

	float4 OutColor = SSSSBlurPS(BufferPos, BufferUV, ViewportDirectionUV, false, SubsurfaceInput0_Extent); 

#if SUBSURFACE_PASS == SUBSURFACE_PASS_TWO
	// second pass prepares the setup from the recombine pass which doesn't need depth but wants to reconstruct the color
	OutColor.a = GetMaskFromDepthInAlpha(OutColor.a);
#endif
	if (all(Pos < Output_ViewportMax))
	{
		SSSColorUAV[Pos] = OutColor;
	}
#endif
}

#endif
 
// Recombines the half res Subsurface filtered lighting contribution (upsampled and renormalized with the alpha)
// with the SceneColor.
void SubsurfaceRecombinePS(in FScreenVertexOutput Input, out float4 OutColor : SV_Target0)
{
	const float2 BufferUV  = Input.UV.xy;

#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(BufferUV);
	const bool bUseSSSDiffusion = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSData.Header);
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const bool bUseSSSDiffusion = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
#endif

	if (!bUseSSSDiffusion)
	{  
		OutColor = Texture2DSample(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, BufferUV);
		return;
	}

#if SUBSTRATE_GBUFFER_FORMAT==1
	const uint ProfileId			= SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header);
	// we multiply the base color later in to get more crips human skin textures (scanned data always has Subsurface included)
	const float3 StoredBaseColor	= SubstrateSubsurfaceExtrasGetBaseColor(SSSData.Extras);
	const float PixelRadiusScale	= SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSData.Header);

	// asset specific color
	float3 Tint = 0.0f;
	float ProfileRadiusScale = 1.0f;
	if (SubstrateSubSurfaceHeaderGetIsProfile(SSSData.Header))
	{
		Tint = GetSubsurfaceProfileKernel(SSSS_TINT_SCALE_OFFSET, ProfileId).rgb;
		ProfileRadiusScale = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + SSSS_N_KERNELWEIGHTCOUNT - 1, ProfileId).a;
	}
	else if (ProfileId == SSS_PROFILE_ID_PERPIXEL)
	{
		// This color is used to lerping 'incident light' into 'diffused light', i.e, controlling the sharpness of the lighting.
		// In case of direct DMPF control, this is not needed.
		Tint = float3(1, 1, 1);
	}
#else
	const uint ProfileId = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
	// we multiply the base color later in to get more crips human skin textures (scanned data always has Subsurface included)
	const float3 StoredBaseColor	= ScreenSpaceData.GBuffer.StoredBaseColor;
	const float PixelRadiusScale	= ScreenSpaceData.GBuffer.CustomData.a;
	// asset specific color
	const float3 Tint	= GetSubsurfaceProfileKernel(SSSS_TINT_SCALE_OFFSET, ProfileId).rgb;
	const float ProfileRadiusScale	= GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + SSSS_N_KERNELWEIGHTCOUNT - 1, ProfileId).a;
#endif

	float3 SSSColor = float3(0, 0, 0);
	float LerpFactor = 1;

#if SUBSURFACE_RECOMBINE_MODE != SUBSURFACE_RECOMBINE_MODE_NO_SCATTERING
#if SUBSURFACE_HALF_RES
	// fade out subsurface scattering if radius is too small to be more crips (not blend with half resolution)
	// minor quality improvement (faces are more detailed in distance)
	LerpFactor = ComputeFullResLerp(ProfileRadiusScale, PixelRadiusScale, BufferUV, SubsurfaceInput1_ExtentInverse);
#endif // SUBSURFACE_HALF_RES

	{
		// subsurface scattering buffer has garbage data outside of viewport bilinear min/max. 
		float2 ClampedBufferUV = clamp(BufferUV, SubsurfaceInput1_UVViewportBilinearMin, SubsurfaceInput1_UVViewportBilinearMax);
		float4 SSSColorWithAlpha = Texture2DSampleLevel(SubsurfaceInput1_Texture, SharedSubsurfaceSampler1, ClampedBufferUV, 0);

		// renormalize to dilate RGB to fix half res upsampling artifacts
		SSSColor = SSSColorWithAlpha.rgb / max(SSSColorWithAlpha.a, 0.00001f);
	}
#else // SUBSURFACE_RECOMBINE_MODE == SUBSURFACE_RECOMBINE_MODE_NO_SCATTERING
	// Scalability requests no Scatter, but we still need to reconstruct a color
	LerpFactor = 0;
#endif // SUBSURFACE_RECOMBINE_MODE

	float3 DirectColor = Texture2DSample(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, BufferUV).rgb;

	bool bIsHalfResUpsample = false;
	if (bIsHalfResUpsample && GetSubsurfaceProfileUseBurley(ProfileId))
	{
	#if SUBSTRATE_GBUFFER_FORMAT==1
		FBurleyParameter BurleyParameter = GetBurleyParameters(SSSData);
	#else
		FBurleyParameter BurleyParameter = GetBurleyParameters(ProfileId, ScreenSpaceData.GBuffer);
	#endif
		
		float Depth = CalcSceneDepth(BufferUV);

		float3 CenterSampleWeight = CalculateCenterSampleWeight(Depth, BurleyParameter);

		SSSColor = lerp(SSSColor,DirectColor,CenterSampleWeight);
	}

	uint ReconstructMethod = (SUBSURFACE_RECOMBINE_QUALITY == SUBSURFACE_RECOMBINE_QUALITY_HIGH) ? 3 : 1;

	SDiffuseAndSpecular DiffuseAndSpecular = ReconstructLighting(BufferUV, ReconstructMethod);
	
	float3 ExtractedNonSubsurface = DiffuseAndSpecular.Specular;

	float3 FadedTint = Tint * LerpFactor;

	// combine potentially half res with full res
	float3 SubsurfaceLighting = lerp(DiffuseAndSpecular.Diffuse, SSSColor, FadedTint);

	OutColor = float4(SubsurfaceLighting * StoredBaseColor + ExtractedNonSubsurface, 0.0f);
}

void SubsurfaceRecombineCopyPS(in FScreenVertexOutput Input, out float4 OutColor : SV_Target0)
{
	const float2 BufferUV = Input.UV.xy;

	OutColor = Texture2DSampleLevel(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, BufferUV, 0);
}

void SubsurfaceViewportCopyPS(noperspective float4 InUV : TEXCOORD0, out float4 OutColor : SV_Target0)
{
	OutColor = Texture2DSample(SubsurfaceInput0_Texture, SharedSubsurfaceSampler0, InUV.xy);
}

#if SUBSURFACE_DEBUG_PASS
#include "ShaderPrint.ush"
#define FontTitle FontEmerald
#define FontValue FontOrange
#define FontBackground float4(0.1,0.1,0.1,0.2)

float ViewResolutionFraction;
int SceneColorHasAlpha;

[numthreads(8, 8, 1)]
void SubsurfaceDebugCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint2 ScreenCoord = DispatchThreadId.xy + View.ViewRectMinAndSize.xy;
	if (any(ScreenCoord >= View.ViewRectMinAndSize.xy + View.ViewRectMinAndSize.zw))
		return;

	float2 BufferUV = (ScreenCoord + 0.5f) * View.BufferSizeAndInvSize.zw;

	uint2 DebugScreenCoordinate = GetCursorPos() * ViewResolutionFraction;
	FShaderPrintContext Ctx = InitShaderPrintContext(all(ScreenCoord == DebugScreenCoordinate), uint2(50, 50));
	if (!Ctx.bIsActive)
	{
		return;
	}

#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(BufferUV);
	const bool bUseSSSDiffusion = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSData.Header);
	const uint ProfileId = SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header);
	const float3 StoredBaseColor = SubstrateSubsurfaceExtrasGetBaseColor(SSSData.Extras);
	const float PixelRadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSData.Header);
	float3 Tint = 0;
	float ProfileRadiusScale = 1.0f;
	if (SubstrateSubSurfaceHeaderGetIsProfile(SSSData.Header))
	{
		Tint = GetSubsurfaceProfileKernel(SSSS_TINT_SCALE_OFFSET, ProfileId).rgb;
		ProfileRadiusScale = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + SSSS_N_KERNELWEIGHTCOUNT - 1, ProfileId).a;
	}
	else if (ProfileId == SSS_PROFILE_ID_PERPIXEL)
	{
		Tint = float3(1, 1, 1);
	}
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const bool bUseSSSDiffusion = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
	const uint ProfileId = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
	const float3 StoredBaseColor = ScreenSpaceData.GBuffer.StoredBaseColor;
	const float PixelRadiusScale = ScreenSpaceData.GBuffer.CustomData.a;
	const float3 Tint = GetSubsurfaceProfileKernel(SSSS_TINT_SCALE_OFFSET, ProfileId).rgb;
	const float ProfileRadiusScale = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + SSSS_N_KERNELWEIGHTCOUNT - 1, ProfileId).a;
#endif

	if (!bUseSSSDiffusion)
	{
		AddTextBackground(Ctx, FontBackground);
		return;
	}

	// --- Reconstruction ---
	float4 CenterSample = SubsurfaceInput0_Texture.SampleLevel(SharedSubsurfaceSampler0, BufferUV, 0);
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	if (bUseSeparatedSubsurfaceDiffuse)
	{
		CenterSample.a = GetSeparatedSubsurfaceDiffuseLuminance(SharedSubsurfaceSampler0, BufferUV);
	}
#endif

	uint ReconstructMethod = (SUBSURFACE_RECOMBINE_QUALITY == SUBSURFACE_RECOMBINE_QUALITY_HIGH) ? 3 : 1;
	SDiffuseAndSpecular DiffuseAndSpecular = ReconstructLighting(BufferUV, ReconstructMethod);

	Print(Ctx, TEXT("=== SSS Debug ==="), FontTitle);
	Newline(Ctx); Newline(Ctx);
	Print(Ctx, TEXT("Coordinate        : ")); Print(Ctx, DebugScreenCoordinate, FontValue, 8); Newline(Ctx);

	{
		Print(Ctx, TEXT("Format            : "));
		if (SceneColorHasAlpha)
		{
			Print(Ctx, TEXT("RGBA "));
		}
		else
		{
			Print(Ctx, TEXT("RGB  "));
		}

    #if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
		if (bUseSeparatedSubsurfaceDiffuse)
		{
			Print(Ctx, TEXT("R16F"));
		}
		else
		{
			Print(Ctx, TEXT("    "));
		}
		#if (!SUBSURFACE_PROFILE_CHECKERBOARD)
			 #if SUBSURFACE_STORE_SPECULAR_LUMINANCE
				if (bUseSeparatedSubsurfaceDiffuse)
				{
					Print(Ctx, TEXT("(s)"));
				}
				else 
			 #endif
				{
					Print(Ctx, TEXT("(d)"));
				}
			 #endif
		 #endif

		#if SUBSURFACE_PROFILE_CHECKERBOARD
			Print(Ctx, TEXT("CheckerBoard")); Newline(Ctx);
		#else
			Newline(Ctx);
		#endif
	}

	Print(Ctx, TEXT("CombinedColor     : ")); Print(Ctx, CenterSample.rgb, FontValue, 8, 4); Newline(Ctx);
    #if !(SUBSURFACE_PROFILE_CHECKERBOARD || SUBSURFACE_CHANNEL_MODE == 0)
	{
		Print(Ctx, TEXT("CombinedLuminance : ")); Print(Ctx, Luminance(CenterSample.rgb), FontValue, 8, 4); Newline(Ctx);
	}
    #else
	{
		Newline(Ctx);
	}
    #endif

#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	if (bUseSeparatedSubsurfaceDiffuse)
	{
		Print(Ctx, TEXT("R16F"));
		#if (!SUBSURFACE_PROFILE_CHECKERBOARD)
			#if SUBSURFACE_STORE_SPECULAR_LUMINANCE
				if (bUseSeparatedSubsurfaceDiffuse)
				{
					Print(Ctx, TEXT("(s)"));
				}
				else 
			#endif
				{
					Print(Ctx, TEXT("(d)"));
				}
		#endif
		Print(Ctx, TEXT("           : "));
	}
	else
#endif
	{
		Print(Ctx, TEXT("SceneColor.a      : "));
	}

	Print(Ctx, CenterSample.a, FontGrey, 8, 4); Newline(Ctx);


	Newline(Ctx);
	Print(Ctx, TEXT("Diffuse           : ")); Print(Ctx, DiffuseAndSpecular.Diffuse, FontGreen, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("Specular          : ")); Print(Ctx, DiffuseAndSpecular.Specular, FontYellow, 8, 4); Newline(Ctx);


	// --- Recombine ---
	Newline(Ctx);
	Print(Ctx, TEXT("=== Recombine ==="), FontTitle);
	Newline(Ctx); Newline(Ctx);


	// Read SSS blur result
	float2 ClampedBufferUV = clamp(BufferUV, SubsurfaceInput1_UVViewportBilinearMin, SubsurfaceInput1_UVViewportBilinearMax);
	float4 SSSColorWithAlpha = Texture2DSampleLevel(SubsurfaceInput1_Texture, SharedSubsurfaceSampler1, ClampedBufferUV, 0);
	float3 SSSColor = SSSColorWithAlpha.rgb / max(SSSColorWithAlpha.a, 0.00001f);

	float LerpFactor = 1;

	float3 FadedTint = Tint * LerpFactor;
	float3 SubsurfaceLighting = lerp(DiffuseAndSpecular.Diffuse, SSSColor, FadedTint);
	float3 FinalColor = SubsurfaceLighting * StoredBaseColor + DiffuseAndSpecular.Specular;

	Print(Ctx, TEXT("ProfileId         : ")); Print(Ctx, ProfileId, FontValue); Newline(Ctx);
	Print(Ctx, TEXT("StoredBaseColor   : ")); Print(Ctx, StoredBaseColor, FontValue, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("Tint              : ")); Print(Ctx, Tint, FontValue, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("LerpFactor        : ")); Print(Ctx, LerpFactor, FontValue, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("SSSColor (blur)   : ")); Print(Ctx, SSSColor, FontCyan, 8, 4); Newline(Ctx);
	Newline(Ctx);
	Print(Ctx, TEXT("SSS Diffuse       : ")); Print(Ctx, SubsurfaceLighting, FontGreen, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("SSS Specular      : ")); Print(Ctx, DiffuseAndSpecular.Specular, FontYellow, 8, 4); Newline(Ctx);
	Print(Ctx, TEXT("Final             : ")); Print(Ctx, FinalColor, FontWhite, 8, 4);
	Newline(Ctx);
	AddTextBackground(Ctx, FontBackground);
}
#endif
