// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessSubsurfaceCommon.usf: Screenspace subsurface scattering common header.
=============================================================================*/

#pragma once

#include "Common.ush"
#include "SubsurfaceProfileCommon.ush"
#include "BurleyNormalizedSSSCommon.ush"

// the same to FSubsurfaceTiles::ETileType
#define SUBSURFACE_TILE_TYPE_AFIS		0
#define SUBSURFACE_TILE_TYPE_SEPARABLE	1
#define SUBSURFACE_TILE_TYPE_PASSTHROUGH	2
#define SUBSURFACE_TILE_TYPE_ALLNONPASSTHROUGH	3
#define SUBSURFACE_TILE_TYPE_ALL		4
#define SUBSURFACE_TILE_TYPE_COUNT		5

#define TYPE_SEPARABLE   0x1
#define TYPE_BURLEY      0x2
#define TYPE_IN_VIEWPORT 0x4

// x:Radius*DistanceToProjectionWindow/KernelSize*0.5, y:DistanceToProjectionWindow, z:OverrideNumSamples, w: MinGenerateMipsTileCount
float4 SubsurfaceParams;

groupshared uint SubsurfaceTypeFlag;

// fetch surface albedo and diffuse mean free path
// The diffuse mean free path is modulated by world scale and a custom term passed in
// which is the opacity texture.
#if SUBSTRATE_GBUFFER_FORMAT==1
FBurleyParameter GetBurleyParameters(FSubstrateSubsurfaceData SSSData)
{
	FBurleyParameter Out = (FBurleyParameter)0;
	if (SubstrateSubSurfaceHeaderGetIsProfile(SSSData.Header))
	{
		uint ProfileId = SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header);
		float ProfileRadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSData.Header);

		Out.SurfaceAlbedo = GetSubsurfaceProfileSurfaceAlbedo(ProfileId);
		Out.DiffuseMeanFreePath = DecodeDiffuseMeanFreePath(GetSubsurfaceProfileDiffuseMeanFreePath(ProfileId));
		Out.WorldUnitScale = DecodeWorldUnitScale(GetSubsurfaceProfileWorldUnitScale(ProfileId)) * BURLEY_CM_2_MM;

		Out.SurfaceOpacity = ProfileRadiusScale;
		Out.DiffuseMeanFreePath *= Out.SurfaceOpacity;
	}
	else if (SubstrateSubSurfaceHeaderGetUseDiffusion(SSSData.Header))
	{
		float3 SSSMFP = SubstrateSubSurfaceHeaderGetMFP(SSSData.Header);
		float3 SSSBaseColor = SubstrateSubsurfaceExtrasGetBaseColor(SSSData.Extras);

		// Decode the MFP stored as Float111110 in a uint over a RGBA8 unorm
		const float3 MFP = SSSMFP * BURLEY_CM_2_MM; // Stored DMFP is in cm, converted into mm

		const float MaxMFP = max3(MFP.x, MFP.y, MFP.z);
		const uint IndexOfMaxMfp = (MFP.r == MaxMFP) ? 0 : ((MFP.g == MaxMFP) ? 1 : 2);

		const float3 DMFP = GetDMFPFromMFPApprox(SSSBaseColor, MFP);

		Out.SurfaceAlbedo = float4(SSSBaseColor, SSSBaseColor[IndexOfMaxMfp]);
		Out.DiffuseMeanFreePath = float4(DMFP, DMFP[IndexOfMaxMfp]);
		Out.WorldUnitScale = 1.0f;	// no world unit scale change.

		// SUBSTRATE_TODO smooth transistion based on pixel footprint w.r.t. MFPRadius and pixel footprint
		Out.SurfaceOpacity = 1.0f;
		Out.DiffuseMeanFreePath *= Out.SurfaceOpacity;
	}
	return Out;
}
#else
FBurleyParameter GetBurleyParameters(uint SubsurfaceProfileInt, FGBufferData GBuffer)
{
	FBurleyParameter Out = (FBurleyParameter)0;
	Out.SurfaceAlbedo = GetSubsurfaceProfileSurfaceAlbedo(SubsurfaceProfileInt);
	Out.DiffuseMeanFreePath = DecodeDiffuseMeanFreePath(GetSubsurfaceProfileDiffuseMeanFreePath(SubsurfaceProfileInt));
	Out.WorldUnitScale = DecodeWorldUnitScale(GetSubsurfaceProfileWorldUnitScale(SubsurfaceProfileInt)) * BURLEY_CM_2_MM;

	// Mask should always be true, we can probably ignore it?
	Out.SurfaceOpacity = UseSubsurfaceProfile(GBuffer.ShadingModelID) ? GBuffer.CustomData.a : 0.0f;
	Out.DiffuseMeanFreePath *= Out.SurfaceOpacity;

	return Out;
}
#endif

// The extent is for the subsurface texture.
float2 CalculateBurleyScale(float WorldUnitScale, float DepthAtCenter, float2 ViewportSize, float2 Extent, float2 ExtentInverse)
{
	float2 BurleyScale = WorldUnitScale;

	float SSSScaleX = SubsurfaceParams.x;
	BurleyScale *= SSSScaleX / DepthAtCenter;

	// cast from cm to mm for depth, and remove the effect of SUBSURFACE_KERNEL_SIZE. 
	BurleyScale *= SUBSURFACE_KERNEL_SIZE / BURLEY_CM_2_MM;

	// account for Screen Percentage/Dyanmic Resolution Scaling
	BurleyScale *= (ViewportSize.x * ExtentInverse.x);
	BurleyScale.y *= (Extent.x * ExtentInverse.y);

	return BurleyScale;
}

// Given the Depth and the BurleyParameter, figure out the actual radius of the center pixel in MM,
// taking into account the depth and screen dimensions.
float CalculateCenterSampleRadiusInMM(FBurleyParameter BurleyParameter, float2 BurleyScale, float2 ExtentInverse)
{
	float DiffuseMeanFreePath = GetDiffuseMeanFreePathForSampling(BurleyParameter.DiffuseMeanFreePath);

	float A = GetComponentForScalingFactorEstimation(BurleyParameter.SurfaceAlbedo);
	float S = GetScalingFactor(A);
	float3 S3D = GetScalingFactor3D(BurleyParameter.SurfaceAlbedo.xyz);

	// In the reference function, UVOffset = BurleyScale * RadiusInMM
	//      float2 UVOffset = BurleyScale*BurleySampleInfo.RadiusInMM;
	// So, given the UV offset, we can find the distance in mm as:
	//      float DistInMM = UvOffset.x/BurleyScale.x + UvOffset.y/BurleyScale.y;
	// But for stability, we can just average them.
	float CenterSampleRadiusInMM = 0.5f * (ExtentInverse.x/BurleyScale.x + ExtentInverse.y/BurleyScale.y);

	return CenterSampleRadiusInMM;
}

float CalculateCenterSampleCdf(FBurleyParameter BurleyParameter, float CenterSampleRadiusInMM)
{
	float DiffuseMeanFreePathForSampling = GetDiffuseMeanFreePathForSampling(BurleyParameter.DiffuseMeanFreePath);
	float A = GetComponentForScalingFactorEstimation(BurleyParameter.SurfaceAlbedo);
	float S = GetScalingFactor(A);

	float D = DiffuseMeanFreePathForSampling / S;
	float CenterSampleRadiusCdf = GetCDF(D.x,CenterSampleRadiusInMM,0);

	return CenterSampleRadiusCdf;
}
