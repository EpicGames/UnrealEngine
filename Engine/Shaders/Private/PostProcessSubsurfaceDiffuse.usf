// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessSubsurfaceLuminance.usf: Initialize the diffuse luminance.
=============================================================================*/

#include "Common.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

Texture2D SceneColorTexture;

// -- PS path driven by tiles.

void SubsurfaceDiffuseInitPassPS(in FScreenVertexOutput Input, out float OutLuminance : SV_Target0)
{
     
#if SUBSURFACE_STORE_SPECULAR_LUMINANCE
	// In this mode, we store specular luminance, which is zero. 
    OutLuminance = 0.0f;
#else
     uint2 Coordinate = uint2(Input.Position.xy);
    float3 SceneColor = SceneColorTexture.Load(int3(Coordinate, 0)).rgb;
    OutLuminance = Luminance(SceneColor);
#endif
}