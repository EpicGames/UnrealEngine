// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessSubsurfaceDiffuse.usf: Accumulate/read diffuse.
=============================================================================*/
#pragma once

#define DIFFUSE_LUMINANCE_READ 0
#define DIFFUSE_LUMINANCE_WRITE 1

// Default to write
#ifndef SUBSURFACE_DIFFUSE_OPERATION
#define SUBSURFACE_DIFFUSE_OPERATION DIFFUSE_LUMINANCE_WRITE
#endif

#ifndef SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
#define SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE 0
#endif

// The subsurface diffuse pass stores specular luminance instead to deal with the error amplification of Diffuseluminance / CombinedLuminance
// when Diffuseluminance is very close to CombinedLuminance and CombinedLuminance is in R11G11B10. 
// Specular reconstruction would have very large error amplification by SceneColor / Specular when diffuse 
// is dominant. E.g., for C = (2.63, 3.00, 3.97), S=(0.036, 0.042, 0.055), C/S = (73, 71, 72).
// The error amplifier can be reduced from x73 to x1 for specular. The relative specular error is reduced
// from around 60% to <2%. This shifts the error to the other end where specular is the major contribution. Under this condition,
// it is less perceptually noticeable.
#ifndef SUBSURFACE_STORE_SPECULAR_LUMINANCE
#define SUBSURFACE_STORE_SPECULAR_LUMINANCE 1
#endif

float EncodeSeparatedSubsurfaceDiffuse(float4 SceneColorWithAlpha)
{
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && SUBSURFACE_STORE_SPECULAR_LUMINANCE
	return Luminance(SceneColorWithAlpha.rgb) - SceneColorWithAlpha.a;
#else
	return SceneColorWithAlpha.a;
#endif
}

float4 DecodeSeparatedSubsurfaceDiffuse(float4 EncodedSceneColorAndAlpha)
{
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && SUBSURFACE_STORE_SPECULAR_LUMINANCE
	float SceneLuminance = Luminance(EncodedSceneColorAndAlpha.rgb);
	return float4(EncodedSceneColorAndAlpha.rgb, SceneLuminance - EncodedSceneColorAndAlpha.a);
#else
	return EncodedSceneColorAndAlpha;
#endif
}

#if SUBSURFACE_DIFFUSE_OPERATION == DIFFUSE_LUMINANCE_WRITE
RWTexture2D<float> RWSeparatedSubsurfaceDiffuseLuminance;
uint bWriteSeparatedSubsurfaceDiffuse;

// Multiply add
void AccumulateSeparatedSubsurfaceDiffuse(uint2 PixelPosition, float DiffuseLuminance, float Multiplier = 1.0f)
{
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	if (bWriteSeparatedSubsurfaceDiffuse)
	{
		float CurrentLuminance = RWSeparatedSubsurfaceDiffuseLuminance[PixelPosition]; 
		RWSeparatedSubsurfaceDiffuseLuminance[PixelPosition] = CurrentLuminance * Multiplier + DiffuseLuminance;
	}
#endif
}

void AccumulateSeparatedSubsurfaceDiffuse(uint2 PixelPosition, float4 SceneColorWithAlpha, float Multiplier = 1.0f)
{
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	if (bWriteSeparatedSubsurfaceDiffuse)
	{
		float CurrentLuminance = RWSeparatedSubsurfaceDiffuseLuminance[PixelPosition]; 
		RWSeparatedSubsurfaceDiffuseLuminance[PixelPosition] = CurrentLuminance * Multiplier + EncodeSeparatedSubsurfaceDiffuse(SceneColorWithAlpha);
	}
#endif
}
#endif

#if SUBSURFACE_DIFFUSE_OPERATION == DIFFUSE_LUMINANCE_READ
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
Texture2D<float> SeparatedSubsurfaceDiffuseLuminance;
uint bUseSeparatedSubsurfaceDiffuse;

float GetSeparatedSubsurfaceDiffuseLuminance(SamplerState Sampler, float2 UVSceneColor)
{
	return SeparatedSubsurfaceDiffuseLuminance.SampleLevel(Sampler, UVSceneColor, 0).r;
}

#endif
#endif