// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessSubsurfaceTile.usf: Screenspace subsurface scattering tile shaders.
=============================================================================*/

#pragma once

#include "Common.ush"

////////////////////////////////////////////////////////////////////////////////////////////////////////////
#ifndef SHADER_TILE_VS
#define SHADER_TILE_VS 0
#endif

#ifndef SUBSURFACE_COMPUTE
#define SUBSURFACE_COMPUTE 0
#endif

#ifndef TILE_CATERGORISATION_SHADER
#define TILE_CATERGORISATION_SHADER 0
#endif

#ifndef SUBSURFACE_HALF_RES
#define SUBSURFACE_HALF_RES 0
#endif

#define LINE_TEXEL_SIZE 1

#if SUBSURFACE_COMPUTE

int2 ViewportSize;
uint Offset;
Buffer<uint> ConditionBuffer;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;

[numthreads(1, 1, 1)]
void BuildIndirectDispatchArgsCS(uint DT_ID : SV_DispatchThreadID)
{
	uint condition = ConditionBuffer[Offset];
	const bool bShouldDispatch = (condition > 0);

	if (bShouldDispatch)
	{
		int2 DestTextureSize = (ViewportSize + SUBSURFACE_TILE_SIZE * LINE_TEXEL_SIZE - 1)
								/ (SUBSURFACE_TILE_SIZE * LINE_TEXEL_SIZE);

		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, DT_ID, DestTextureSize.x, DestTextureSize.y, 1);
	}
	else
	{
		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, DT_ID, 0, 0, 0);
	}
}

uint2 TextureExtent;
uint2 ViewportMin;
RWTexture2D<float4> TextureOutput;

[numthreads(SUBSURFACE_TILE_SIZE, SUBSURFACE_TILE_SIZE, 1)]
void ClearUAV(uint2 DT_ID : SV_DispatchThreadID)
{
	uint2 BufferPos = ViewportMin + DT_ID * LINE_TEXEL_SIZE;

	if (all(BufferPos < TextureExtent))
	{
		UNROLL
		for (uint i = 0; i < LINE_TEXEL_SIZE; ++i)
		{
			UNROLL
			for (uint j = 0; j < LINE_TEXEL_SIZE; ++j)
			{
				uint2 CurrentBufferPos = min(BufferPos + uint2(i, j), TextureExtent - uint2(1,1));
				TextureOutput[CurrentBufferPos] = float4(0.0f, 0.0f, 0.0f, 0.0f);
			}
		}
	}
}

#endif

#if SHADER_TILE_VS

int2 ViewMin;
int2 ViewMax;
float2 ExtentInverse;
uint TileType;
Buffer<uint> TileDataBuffer;

void MainVS(
	in uint InVertexId : SV_VertexID,
	in uint InInstanceId : SV_InstanceID,
	out FScreenVertexOutput Out)
{
	Out = (FScreenVertexOutput)0;
	const uint2 TileCoord = UnpackTileCoord12bits(TileDataBuffer[InInstanceId]);

	uint2 TileVertex = TileCoord * SUBSURFACE_TILE_SIZE;
	TileVertex.x += InVertexId == 1 || InVertexId == 2 || InVertexId == 4 ? SUBSURFACE_TILE_SIZE : 0;
	TileVertex.y += InVertexId == 2 || InVertexId == 4 || InVertexId == 5 ? SUBSURFACE_TILE_SIZE : 0;
	Out.UV = float2(min((int2)TileVertex + ViewMin, ViewMax)) * ExtentInverse;
	Out.Position = float4(Out.UV * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.0f, 1.0f);
}

#endif //SHADER_TILE_VS

void SubsurfaceTileFallbackScreenPassVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out FScreenVertexOutput Out)
{
	DrawRectangle(InPosition, InTexCoord, Out.Position, Out.UV);
}

//Code adapted from DefaultSSRTiles.usf
#if TILE_CATERGORISATION_SHADER

#include "Substrate/Substrate.ush"
#include "Substrate/SubstrateEvaluation.ush"

#include "PostProcessCommon.ush"
#include "DeferredShadingCommon.ush"
#include "PostProcessSubsurfaceCommon.ush"
#include "ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(SubsurfaceInput0)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

groupshared uint2 ContainsSSS[SUBSURFACE_TILE_SIZE * SUBSURFACE_TILE_SIZE];
groupshared uint SubsurfacePassthroughFlag;

int MaxGroupCount;
int2 TiledViewRes;
float SubsurfaceSubpixelThreshold;

RWStructuredBuffer<uint> TileMaskBufferOut;
RWStructuredBuffer<uint> TileMaskPassthroughBufferOut;
RWStructuredBuffer<uint> TileMaskNonPassthroughBufferOut;

#if SUBSTRATE_GBUFFER_FORMAT==1
//Duplicate from PostProcessSubsurface.usf
FSubstrateSubsurfaceData LoadSubstrateSSSData(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateLoadSubsurfaceData(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelPos);
}
FSubstrateSubsurfaceHeader LoadSubstrateSSSHeader(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelPos);
}
//Duplicate from SubsurfaceBurleyNormalized.ush
FSubstrateTopLayerData LoadSubstrateTopLayerData(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelPos, 0)));
}
#endif

// Only access shading model GBuffer
float HasSubsurfaceForOnePixel(float2 UVSceneColor)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(UVSceneColor);
	const bool bHasSSSDiffusion = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSHeader);
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UVSceneColor);
	const bool bHasSSSDiffusion = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);
#endif

	return select(bHasSSSDiffusion, 1.0f, 0.0f);
}

bool IsEffectiveSSS(float2 BufferUV, float HasSSSDiffusion)
{
	if (HasSSSDiffusion == 0.0f)
	{
		return false;
	}

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(BufferUV);
	const FSubstrateTopLayerData TopLayerData = LoadSubstrateTopLayerData(BufferUV);
	const uint SubsurfaceProfileInt = SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header);
	const FBurleyParameter BurleyParameter = GetBurleyParameters(SSSData);
#else
	const FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
	const FBurleyParameter BurleyParameter = GetBurleyParameters(SubsurfaceProfileInt, ScreenSpaceData.GBuffer);
#endif

	float DepthAtCenter = CalcSceneDepth(BufferUV);

	if (DepthAtCenter <= 0)
	{
		return false;
	}

	float2 BurleyScale = CalculateBurleyScale(BurleyParameter.WorldUnitScale, DepthAtCenter, Output_ViewportSize, Output_Extent, Output_ExtentInverse);
	float CenterSampleRadiusInMM = CalculateCenterSampleRadiusInMM(BurleyParameter, BurleyScale, Output_ExtentInverse);
	float CenterSampleCdf = CalculateCenterSampleCdf(BurleyParameter, CenterSampleRadiusInMM);

	// ignore the pixel if the scattering is inside itself.
	return select(CenterSampleCdf < SubsurfaceSubpixelThreshold, true, false);
}

[numthreads(SUBSURFACE_TILE_SIZE, SUBSURFACE_TILE_SIZE, 1)]
void SSSTileCategorisationMarkCS(uint3 DT_ID : SV_DispatchThreadID, 
	uint3 G_ID : SV_GroupID, 
	uint3 GT_ID : SV_GroupThreadID,
	uint  GI : SV_GroupIndex)
{
	if (GI == 0)
	{
		SubsurfaceTypeFlag = 0;
		SubsurfacePassthroughFlag = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	uint2 Pos = DT_ID.xy + Output_ViewportMin;
	float2 BufferUV = Output_ExtentInverse * (float2(Pos) + 0.5f);
	uint Type = 0;
	
#if SUBSURFACE_HALF_RES
	float2 UVA = min(BufferUV + float2(-0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
	float2 UVB = min(BufferUV + float2( 0.5, 0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
	float2 UVC = min(BufferUV + float2( 0.5,-0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);
	float2 UVD = min(BufferUV + float2(-0.5,-0.5f) * SubsurfaceInput0_ExtentInverse, SubsurfaceInput0_UVViewportBilinearMax);

	float A = HasSubsurfaceForOnePixel(UVA);
	float B = HasSubsurfaceForOnePixel(UVB);
	float C = HasSubsurfaceForOnePixel(UVC);
	float D = HasSubsurfaceForOnePixel(UVD);

	float Sum = (A + B) + (C + D);
	// Each alpha value of A/B/C/D should be either 1.0f or 0.0, so a check of 0.5 should be safe
	bool bHasSubsurface = (Sum >= .50f);
#else
	float A = HasSubsurfaceForOnePixel(BufferUV);
	bool bHasSubsurface = bool(A);
#endif
	bHasSubsurface = bHasSubsurface && all(Pos < Output_ViewportMax);
	bool bIsPassthroughPixel = false;

	BRANCH
	if (bHasSubsurface)
	{
#if SUBSURFACE_HALF_RES
		bool bEffectiveSSSA = IsEffectiveSSS(UVA, A);
		bool bEffectiveSSSB = IsEffectiveSSS(UVB, B);
		bool bEffectiveSSSC = IsEffectiveSSS(UVC, C);
		bool bEffectiveSSSD = IsEffectiveSSS(UVD, D);

		bIsPassthroughPixel = !((bEffectiveSSSA | bEffectiveSSSB | bEffectiveSSSC | bEffectiveSSSD));
#else
		bool bEffectiveSSS = IsEffectiveSSS(BufferUV, A);
		bIsPassthroughPixel = !bEffectiveSSS;
#endif
	}

	bool2 bWriteTile = false;

#if COMPILER_SUPPORTS_WAVE_VOTE
	const uint NumOfActiveSSSPixelsInWave = WaveActiveCountBits(bHasSubsurface);
	const uint NumOfActivePassthroughPixdelsInWave = WaveActiveCountBits(bIsPassthroughPixel);
	if (WaveIsFirstLane() && NumOfActiveSSSPixelsInWave)
	{
		uint IndexToStore = 0;
		InterlockedAdd(SubsurfaceTypeFlag, NumOfActiveSSSPixelsInWave, IndexToStore);
	}

	if (WaveIsFirstLane() && NumOfActivePassthroughPixdelsInWave > 0)
	{
		uint IndexToStore = 0;
		InterlockedAdd(SubsurfacePassthroughFlag, NumOfActivePassthroughPixdelsInWave, IndexToStore);
	}

	GroupMemoryBarrierWithGroupSync();
	if (GI == 0)
	{
		bWriteTile = bool2(SubsurfaceTypeFlag > 0 ,
			               SubsurfacePassthroughFlag == SubsurfaceTypeFlag);
	}
#else

	const int LinearIndex = GT_ID.y * SUBSURFACE_TILE_SIZE + GT_ID.x;
	ContainsSSS[LinearIndex] = uint2(bHasSubsurface,bIsPassthroughPixel);
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 32) // 8*8 = 64 elements to merge
	{
		ContainsSSS[LinearIndex] = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		ContainsSSS[LinearIndex] = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	// The smallest wave size is 16 on Intel hardware. So now we can do simple math operations without group sync.
	// EDIT: for some reason group sync is needed until the end, otherwise some pixels are missing...

	if (LinearIndex < 8)
	{
		ContainsSSS[LinearIndex] = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		ContainsSSS[LinearIndex] = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		ContainsSSS[LinearIndex] = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 1)
	{
		uint2 ContainsSSSResult = ContainsSSS[LinearIndex] + ContainsSSS[LinearIndex + 1];
		bWriteTile.x = ContainsSSSResult.x > 0;
		bWriteTile.y = ContainsSSSResult.x ==  ContainsSSSResult.y;
	}
#endif

	// Set bit to indicate tile is occupied.
	uint MaskLinearIndex = TiledViewRes.x * G_ID.y + G_ID.x;

	// All subsurface tiles
	if (bWriteTile.x)
	{	
		InterlockedOr(TileMaskBufferOut[MaskLinearIndex / 32U], 1U << (MaskLinearIndex % 32U));
	}

	// passthrough tiles
	if (bWriteTile.x && bWriteTile.y)
	{
		InterlockedOr(TileMaskPassthroughBufferOut[MaskLinearIndex / 32U], 1U << (MaskLinearIndex % 32U));
	}

	// non-passthrough subsurface tiles
	if (bWriteTile.x && !bWriteTile.y)
	{
		InterlockedOr(TileMaskNonPassthroughBufferOut[MaskLinearIndex / 32U], 1U << (MaskLinearIndex % 32U));
	}

}

int TileType;
StructuredBuffer<uint> TileMaskBuffer;

RWBuffer<uint> RWSSSTileListData;
RWBuffer<uint> RWTileTypeCountBuffer;
RWBuffer<uint> RWSSSTileListDataBuffer;

groupshared uint SharedNumTiles;
groupshared uint SharedTileData[SUBSURFACE_TILE_SIZE * SUBSURFACE_TILE_SIZE];
groupshared uint SharedGlobalTileOffset;

/**
* Every group checks 64 tiles and builds a spatially coherent compacted list of SSS tiles
*/
[numthreads(SUBSURFACE_TILE_SIZE, SUBSURFACE_TILE_SIZE, 1)]
void SSSTileClassificationBuildListsCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID)
{
	const uint LinearThreadIndex = GroupThreadId.y * SUBSURFACE_TILE_SIZE + GroupThreadId.x;

	if (LinearThreadIndex == 0)
	{
		SharedNumTiles = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex == 0)
	{
		for (uint LocalTileIndex = 0; LocalTileIndex < SUBSURFACE_TILE_SIZE * SUBSURFACE_TILE_SIZE; ++LocalTileIndex)
		{
			// ZOrder tiles to maximize screen locality after converting to 1d for compaction
			// The tile locality ultimately affects gathering from neighbor tiles.
			uint2 ThreadOffset = ZOrder2D(LocalTileIndex, log2(SUBSURFACE_TILE_SIZE));
			uint2 TileCoordinate = GroupId * SUBSURFACE_TILE_SIZE + ThreadOffset;

			if (all(TileCoordinate < TiledViewRes))
			{
				uint MaskLinearIndex = TiledViewRes.x * TileCoordinate.y + TileCoordinate.x;
				uint Mask = 1u << (MaskLinearIndex % 32u);
				bool bTileUsed = (TileMaskBuffer[MaskLinearIndex / 32u] & Mask) != 0;
				if (bTileUsed)
				{
					uint TileOffset = SharedNumTiles;
					SharedTileData[TileOffset] = PackTileCoord12bits(TileCoordinate);
					SharedNumTiles = TileOffset + 1;
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Allocate space in the tile list
	if (LinearThreadIndex == 0 && SharedNumTiles > 0)
	{
		InterlockedAdd(RWTileTypeCountBuffer[TileType], SharedNumTiles, SharedGlobalTileOffset);
	}

	GroupMemoryBarrierWithGroupSync();

	// Write out tiles
	if (LinearThreadIndex < SharedNumTiles)
	{
		RWSSSTileListDataBuffer[SharedGlobalTileOffset + LinearThreadIndex] = SharedTileData[LinearThreadIndex];
	}
}

int VertexCountPerInstanceIndirect;
Buffer<uint> TileTypeCountBuffer;
RWBuffer<uint> RWIndirectDrawArgsBuffer;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;

[numthreads(8, 1, 1)]
void SubsurfaceTileBuildIndirectDispatchArgsCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.y == 0 && DispatchThreadId.x < SUBSURFACE_TILE_TYPE_COUNT 
	    && (DispatchThreadId.x == SUBSURFACE_TILE_TYPE_PASSTHROUGH
		|| DispatchThreadId.x == SUBSURFACE_TILE_TYPE_ALLNONPASSTHROUGH
		|| DispatchThreadId.x == SUBSURFACE_TILE_TYPE_ALL))
	{
		uint TileType = DispatchThreadId.x;
		const uint TileCount = TileTypeCountBuffer[TileType];

		// Indirect draw
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 0] = VertexCountPerInstanceIndirect;  // VertexCountPerInstance
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 1] = TileCount;  // InstanceCount
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 2] = 0;  // StartVertexLocation
		RWIndirectDrawArgsBuffer[TileType * DRAW_INDIRECT_UINT_COUNT + 3] = 0;  // StartInstanceLocation

		// Indirect dispatch
		const uint IndirectDispatchSize = TileCount;
		WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, TileType, IndirectDispatchSize, 1, 1);
	}
}

#endif