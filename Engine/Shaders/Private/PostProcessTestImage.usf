// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessTestImage.usf: PostProcessing shader to show a test image
=============================================================================*/

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "DeferredShadingCommon.ush"
#include "ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Output)

uint FrameNumber;
float FrameTime;

float ComputeDistanceToViewRect(int2 PixelPos, int Border)
{
	return ComputeDistanceToRect(PixelPos, int2(Output_ViewportMin) + Border, int2(Output_ViewportSize) - Border * 2, false);
}

float Quantize(float x, float Count)
{
	return ceil(x * Count - 1.0f) / (Count - 1.0f);
}

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	float2 UV = UVAndScreenPos.xy;
	// full res
	int2 PixelPos = (int2)SvPosition.xy;
	// 1/2 res
	int2 PixelPos2 = PixelPos / 2;
	// 1/32 res
	int2 PixelPos6 = PixelPos / 32;
	//
	int2 ScreenCenter = (int2)(Output_ViewportMin + Output_ViewportSize / 2);

	// highest frequency dither pattern (2x2)
	float BlackWhiteDither1 = ((uint)PixelPos.x % 2) == ((uint)PixelPos.y % 2);
	// checker board pattern (4x4)
	float BlackWhiteDither2 = ((uint)PixelPos2.x % 2) == ((uint)PixelPos2.y % 2);
	// checker board pattern (4x4)
	float BlackWhiteDither6 = ((uint)PixelPos6.x % 2) == ((uint)PixelPos6.y % 2);
	
	// grey panning bars on grey background to see VSync issues
	{
		// no antialiasing to avoid changes when upscaling
		float Mask = saturate(1 - ComputeDistanceToRect(PixelPos, (int2)(Output_ViewportMin + Output_ViewportSize * float2(0.9f, 0.0f)), int2(Output_ViewportSize * float2(0.1f, 1.0f))));

		// speed - higher value is more useful to see VSync issues
		const uint PixelsPerFrame = 8;
			
		bool bLine = ((FrameNumber * PixelsPerFrame) - PixelPos.x) % 64 > 32;
		OutColor = bLine ? float4(0.5f, 0.5f, 0.5f, 0) : float4(0.2f, 0.2f, 0.2f, 0);
	}

	// 4 white pixel wide lines with a black line between
	float ViewportBorders;
	{
		uint ViewPortRectDistance = (uint)ComputeDistanceToViewRect(PixelPos, 7) + 7;

		ViewportBorders = (ViewPortRectDistance % 2) == 0;
	}
	OutColor += ViewportBorders;
	
	{
		float CenterDist = ComputeDistanceToRect(PixelPos, ScreenCenter, 0);
		
		// round safety border
		{
			float CircleSize = min(Output_ViewportSize.x, Output_ViewportSize.y) / 2 * 0.78f / 16;

			float SafetyRectDist = ComputeDistanceToRect(PixelPos, (int2)(Output_ViewportMin + Output_ViewportSize * 0.1f), (int2)(Output_ViewportSize * 0.8f));

			float InnerCircleThickness = 1.0f;
			float OuterCircleThickness = 7.0f;
			float ThinCircleMask = saturate(SafetyRectDist - CircleSize) * (1 - saturate(SafetyRectDist - CircleSize - InnerCircleThickness));
			float ThickCircleMask =  (1 - saturate(SafetyRectDist - CircleSize - InnerCircleThickness - OuterCircleThickness));

			OutColor = lerp(OutColor, ThinCircleMask.xxxx * float4(0.5f, 0.5f, 0.5f, 0.5f), ThickCircleMask.xxxx);
		}

		// medium circle
		{
			float CircleSize = min(Output_ViewportSize.x, Output_ViewportSize.y) / 2 * 0.78f / 2;

			// we want to see good antialiasing but not too blurry so we can judge artifacts
			float InnerCircleThickness = 1.0f;
			float ThinCircleMask = saturate(CenterDist - CircleSize) * (1 - saturate(CenterDist - CircleSize - InnerCircleThickness));
			float ThickCircleMask = saturate(CenterDist - CircleSize + 3) * (1 - saturate(CenterDist - CircleSize - InnerCircleThickness - 3));

			OutColor = lerp(OutColor, ThinCircleMask.xxxx, ThickCircleMask.xxxx);
		}

		// smallest circle
		{
			float CircleSize = min(Output_ViewportSize.x, Output_ViewportSize.y) / 2 * 0.78f / 16;

			// we want to see good antialiasing but not too blurry so we can judge artifacts
			float InnerCircleThickness = 1.0f;
			float ThinCircleMask = saturate(CenterDist - CircleSize) * (1 - saturate(CenterDist - CircleSize - InnerCircleThickness));
			float ThickCircleMask = saturate(CenterDist - CircleSize + 3) * (1 - saturate(CenterDist - CircleSize - InnerCircleThickness - 3));

			OutColor = lerp(OutColor, ThinCircleMask.xxxx, ThickCircleMask.xxxx);
		}

		// Red green blue
		{
			// no antialiasing to avoid changes when upscaling
			int RedMask = ComputeDistanceToRect(PixelPos, (int2)(Output_ViewportMin + Output_ViewportSize * float2(0.1f, 0.45f)), (int2)(Output_ViewportSize * float2(0.04f, 0.1f)));
			int GreenMask = ComputeDistanceToRect(PixelPos, (int2)(Output_ViewportMin + Output_ViewportSize * float2(0.15f, 0.45f)), (int2)(Output_ViewportSize * float2(0.04f, 0.1f)));
			int BlueMask = ComputeDistanceToRect(PixelPos, (int2)(Output_ViewportMin + Output_ViewportSize * float2(0.2f, 0.45f)), (int2)(Output_ViewportSize * float2(0.04f, 0.1f)));

			OutColor = lerp(OutColor, float4(1, 0, 0, 0), (float)(RedMask < 1));
			OutColor = lerp(OutColor, float4(0, 1, 0, 0), (float)(GreenMask < 1));
			OutColor = lerp(OutColor, float4(0, 0, 1, 0), (float)(BlueMask < 1));

			if(RedMask == 2 || GreenMask == 2 || BlueMask == 2)
			{
				// white border
				OutColor = 1;
			}
		}
	}
		
	// top 3 bars
	{
		uint2 LeftTopBars = uint2(Output_ViewportMin + Output_ViewportSize * float2(0.5f, 0.1f));
		int2 SizeBars = (int2)(Output_ViewportSize * float2(0.4f, 0.05f));
		
		bool FirstBarMask =  ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.00f)), SizeBars, false) == 0;
		bool SecondBarMask = ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.06f)), SizeBars, false) == 0;
		bool ThirdBarMask =  ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.12f)), SizeBars, false) == 0;

		float Fraction = saturate((PixelPos.x - LeftTopBars.x) / (SizeBars.x - 1.0f));

		// moving bars (ideally little affected by hitches and framerate)
		float BarState = frac(FrameTime - PixelPos.x / 64.0f) < 0.5f;
		OutColor = lerp(OutColor, float4(BarState, BarState, 1.0f, 0), FirstBarMask);
		// finest checkerboard pattern
		OutColor += BlackWhiteDither1 * SecondBarMask;
		// more coarse checkerboard pattern
		OutColor += BlackWhiteDither2 * ThirdBarMask;
	}

	// bottom 3 bars
	{
		uint2 LeftTopBars = uint2(Output_ViewportMin + Output_ViewportSize * float2(0.1f, 0.85f - 0.12f));
		int2 SizeBars = (int2)(Output_ViewportSize * float2(0.8f, 0.05f));
		
		bool FirstBarMask =  ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.00f)), SizeBars, false) == 0;
		bool SecondBarMask = ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.06f)), SizeBars, false) == 0;
		bool ThirdBarMask =  ComputeDistanceToRect(PixelPos, (int2)(LeftTopBars + Output_ViewportSize * float2(0.0f, 0.12f)), SizeBars, false) == 0;

		float Fraction = saturate((PixelPos.x - LeftTopBars.x) / (SizeBars.x - 1.0f));

		// bright greyscale blocks (255 - 19 is the white reference)
		OutColor += lerp(255 - 2 * 19, 255, Quantize(Fraction, 8)) / 255.0f * FirstBarMask;
		// dark distinct greyscale blocks (16 is the black reference)
		OutColor += lerp(0, 2 * 16, Quantize(Fraction, 8)) / 255.0f * SecondBarMask;
		// greyscale gradient
		OutColor += Fraction * ThirdBarMask;
	}

	OutColor.rgb = ColorCorrection(OutColor.rgb);
}