// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessTonemap.usf: PostProcessing tone mapping  
=============================================================================*/

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#ifndef SUPPORTS_SCENE_COLOR_APPLY_PARAMETERS
	#define Undefined SUPPORTS_SCENE_COLOR_APPLY_PARAMETERS 0
#endif

#define LOCAL_EXPOSURE_MODE_NONE 0
#define LOCAL_EXPOSURE_MODE_BILATERAL 1
#define LOCAL_EXPOSURE_MODE_FUSION 2

#ifndef USE_VIGNETTE_TEXTURE
#define USE_VIGNETTE_TEXTURE 0
#endif

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "TonemapCommon.ush"
#include "EyeAdaptationCommon.ush"
#include "PostProcessHistogramCommon.ush"
#include "PixelQuadMessagePassing.ush"
#include "ScreenPass.ush"

#define DEBUG_GRAIN_QUANTIZATION 0

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

#if MOBILE_MULTI_VIEW
Texture2DArray ColorTexture;
#else
Texture2D ColorTexture;
#endif
SamplerState ColorSampler;


FScreenTransform ColorToBloom;
float2 BloomUVViewportBilinearMin;
float2 BloomUVViewportBilinearMax;

#if MOBILE_MULTI_VIEW
	Texture2DArray BloomTexture;
#else
	Texture2D BloomTexture;
#endif
SamplerState BloomSampler;

#if SUPPORTS_SCENE_COLOR_APPLY_PARAMETERS
	StructuredBuffer<float4> SceneColorApplyParamaters;
#endif

#if LOCAL_EXPOSURE_MODE == LOCAL_EXPOSURE_MODE_BILATERAL
Texture3D LumBilateralGrid;
Texture2D BlurredLogLum;
SamplerState LumBilateralGridSampler;
SamplerState BlurredLogLumSampler;
#elif LOCAL_EXPOSURE_MODE == LOCAL_EXPOSURE_MODE_FUSION
FScreenTransform ColorToExposureFusion;
SCREEN_PASS_TEXTURE_VIEWPORT(ExposureFusion)
Texture2D ExposureFusionTexture;
SamplerState ExposureFusionSampler;
#endif

// xyz:SceneColorTint.rgb, w:unused
float4 ColorScale0;

// xyz:Bloom1Tint.rgb, w:unused
float4 ColorScale1;

float SharpenDiv6;
uint Flags;

float4 VignetteParams0;
float4 VignetteParams1;
float  VignetteParams2;
uint   VignetteType;
#if USE_VIGNETTE_TEXTURE
Texture2D VignetteTexture;
SamplerState VignetteTextureSampler;
#endif

// Grain quantization
float3 GrainRandomFull;

// Film grain
float FilmGrainIntensityShadows;
float FilmGrainIntensityMidtones;
float FilmGrainIntensityHighlights;
float FilmGrainShadowsMax;
float FilmGrainHighlightsMin;
float FilmGrainHighlightsMax;
Texture2D<half3> FilmGrainTexture;
SamplerState FilmGrainSampler;
float4 ScreenPosToFilmGrainTextureUV;
#if SUPPORTS_FILM_GRAIN
	StructuredBuffer<float4> FilmGrainTextureConstants;
#endif // SUPPORTS_FILM_GRAIN

float BackbufferQuantizationDithering;
float EditorNITLevel;

uint OutputDevice;
uint OutputGamut;
float OutputMaxLuminance;
uint LUTShaper;

float4 BloomDirtMaskTint;
Texture2D BloomDirtMaskTexture;
SamplerState BloomDirtMaskSampler;

float4 LensPrincipalPointOffsetScale;
float4 LensPrincipalPointOffsetScaleInverse;

uint GetOutputDevice()
{
#if OUTPUT_DEVICE_SRGB
	return TONEMAPPER_OUTPUT_sRGB;
#else
	return OutputDevice;
#endif
}

half GrainFromUV(float2 GrainUV) 
{
	half Grain = frac(sin(GrainUV.x + GrainUV.y * 543.31) *  493013.0);
	return Grain;
}

// converts from screen [-1,1] space to the lens [-1,1] viewport space
float2 ConvertScreenViewportSpaceToLensViewportSpace(float2 UV)
{
	return LensPrincipalPointOffsetScale.xy + UV * LensPrincipalPointOffsetScale.zw;
}

float2 ConvertLensViewportSpaceToScreenViewportSpace(float2 UV)
{
	// reference version
	//return (UV - LensPrincipalPointOffsetScale.xy)/LensPrincipalPointOffsetScale.zw;

	// optimized version
	return LensPrincipalPointOffsetScaleInverse.xy + UV * LensPrincipalPointOffsetScaleInverse.zw;
}


// Nuke-style Color Correct
float ComputeFilmGrainIntensity(float3 LinearColorRGBGamut)
{
	const float3x3 sRGB_2_AP1 = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );
	const float3x3 AP1_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );

	float3 ColorAP1 = mul(sRGB_2_AP1, LinearColorRGBGamut);
	
	//float Luma = dot(ColorAP1, AP1_RGB2Y);
	float Luma = dot(LinearColorRGBGamut,  mul(AP1_2_sRGB, AP1_RGB2Y));

	float CCWeightShadows = 1 - smoothstep(0, FilmGrainShadowsMax, Luma);
	float CCWeightHighlights = smoothstep(FilmGrainHighlightsMin, FilmGrainHighlightsMax, Luma);
	float CCWeightMidtones = 1 - CCWeightShadows - CCWeightHighlights;
	
	// Blend Shadow, Midtone and Highlight CCs
	float FilmGrainIntensity = (
		FilmGrainIntensityShadows * CCWeightShadows +
		FilmGrainIntensityMidtones * CCWeightMidtones +
		FilmGrainIntensityHighlights * CCWeightHighlights);
	
	return FilmGrainIntensity;
}

bool IsFilmGrainEnabled()
{
	return FilmGrainIntensityShadows != 0 || FilmGrainIntensityMidtones != 0 || FilmGrainIntensityHighlights != 0;
}


// LUT for color grading
Texture3D ColorGradingLUT;
SamplerState ColorGradingLUTSampler;

float LUTMax;
float LUTSize;
float InvLUTSize; // 1 / LUTSize
float LUTScale; // (LUTSize - 1) / LUTSize
float LUTOffset; // 0.5 / LUTSize

half3 ColorLookupTable( half3 LinearColor )
{
	float3 LUTEncodedColor;
	// Encode as ST-2084 (Dolby PQ) values
	BRANCH
	if (LUTShaper)
	{
		// ST2084 expects to receive linear values 0-10000 in nits.
		// So the linear value must be multiplied by a scale factor to convert to nits.
		// We don't send negative values to LinearToST2084 as it will result in NaN because of pow.
		LUTEncodedColor = LinearToST2084(clamp(LinearColor * LinearToNitsScale, 0.0, 10000.0));
	}
	else
	{
		LUTEncodedColor = LinToLog( LinearColor + LogToLin( 0 ) );
	}

	float3 UVW = LUTEncodedColor * LUTScale + LUTOffset;

	half3 OutDeviceColor = Texture3DSample( ColorGradingLUT, ColorGradingLUTSampler, UVW ).rgb;
	return OutDeviceColor * LUTMax;
}

// can be optimized
float2 ScreenPosToUV(float2 ScreenPos, float2 ExtentInverse)
{
	float2 UV = (ScreenPos * Color_ScreenPosToViewportScale + Color_ScreenPosToViewportBias) * ExtentInverse;

	return UV;
}

float2 UVToScreenPos(float2 UV, float2 Extent)
{
	return (UV * Extent - Color_ScreenPosToViewportBias) / Color_ScreenPosToViewportScale;
}

float4 ChromaticAberrationParams;

void TonemapCommonVS(
	in	float4 Position,
	in	float2 TexCoord,
	out	float2 OutVignette,
	out	float4 OutGrainUV,
	out	float2 OutScreenPos,
	out	float2 OutFullViewUV
	)
{
	// Forward renderer uses view size texture
	// TODO: Looks to be Ronin specific.. check this:
	OutFullViewUV.xy = Position.xy * float2(0.5,-0.5) + 0.5;

	const float AspectRatio = Output_ViewportSize.y * Output_ViewportSizeInverse.x;

	float2 ColorViewportPos = UVToScreenPos(TexCoord, Color_Extent);

	// Scale vignette to always be a circle with consistent corner intensity.
	float2 LensViewportPos = ConvertScreenViewportSpaceToLensViewportSpace(ColorViewportPos);
	OutVignette = VignetteSpace(LensViewportPos, VignetteType, AspectRatio);

	// Grain
	OutGrainUV.xy = TexCoord + Color_ExtentInverse * float2(-0.5,0.5);
	#if DEBUG_GRAIN_QUANTIZATION
		OutGrainUV.zw = TexCoord;
	#else
		OutGrainUV.zw = TexCoord + GrainRandomFull.xy;
	#endif

	// Fringe
	OutScreenPos = UVToScreenPos(TexCoord, Color_Extent);
}

// vertex shader entry point
void MainVS(
	in 					float4 InPosition 				: ATTRIBUTE0,
	in 					float2 InTexCoord 				: ATTRIBUTE1,
	out noperspective 	float2 OutTexCoord 				: TEXCOORD0,
	out noperspective 	float2 OutVignette				: TEXCOORD1,
	out noperspective 	float4 OutGrainUV 				: TEXCOORD2,
	out noperspective 	float2 OutScreenPos 			: TEXCOORD3,
	out noperspective 	float2 OutFullViewUV 			: TEXCOORD4,
	in FStereoVSInput StereoInput,
	out FStereoVSOutput StereoOutput,
	out 				float4 OutPosition 				: SV_POSITION
	)
{
	StereoSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutTexCoord);
    TonemapCommonVS(OutPosition, OutTexCoord, OutVignette, OutGrainUV, OutScreenPos, OutFullViewUV);
}

float max4(float4 x)
{
	return max(max(x.r, x.g), max(x.b, x.a));
}

float4 SampleSceneColor(float2 SceneUV, float ArrayIndex)
{
	SceneUV = clamp(SceneUV, Color_UVViewportBilinearMin, Color_UVViewportBilinearMax);
#if MOBILE_MULTI_VIEW
	return Texture2DArraySample(ColorTexture, ColorSampler, float3(SceneUV.xy,ArrayIndex));
#else
	return Texture2DSample(ColorTexture, ColorSampler, SceneUV);
#endif
}

half3 LookupSceneColor(float2 SceneUV, float ArrayIndex, float2 PixelOffset)
{
	float2 SampleUV = SceneUV + PixelOffset * Color_ExtentInverse;
	return SampleSceneColor(SampleUV, ArrayIndex).xyz;
}

float4 TonemapCommonPS(
	float3 UV,
	float2 Vignette,
	float4 GrainUV,
	float2 ScreenPos, // [-1, 1]x[-1, 1]
	float2 FullViewUV,
	float4 SvPosition,
	out float OutLuminance
	)
{
	const float OneOverPreExposure = View.OneOverPreExposure;
	const float2 EyeAdaptationData = EyeAdaptationBuffer[0].xw;
	const float GlobalExposure = EyeAdaptationData.x;

	float2 SceneUV = UV.xy;
	float ArrayIndex = UV.z;

	// Sample the scene color.
	half4 SceneColor;
	{
		#if USE_COLOR_FRINGE
			float2 SceneUVJitter = float2(0.0, 0.0);
		
				float2 CAScale = ChromaticAberrationParams.rg;
				float StartOffset = ChromaticAberrationParams.z;

				float2 LensUV = ConvertScreenViewportSpaceToLensViewportSpace(ScreenPos);

				float4 CAUV;
				CAUV = LensUV.xyxy - sign(LensUV).xyxy * saturate(abs(LensUV) - StartOffset).xyxy * CAScale.rrgg;

				CAUV.xy = ConvertLensViewportSpaceToScreenViewportSpace(CAUV.xy);
				CAUV.zw = ConvertLensViewportSpaceToScreenViewportSpace(CAUV.zw);

				CAUV.xy = ScreenPosToUV(CAUV.xy, Color_ExtentInverse);
				CAUV.zw = ScreenPosToUV(CAUV.zw, Color_ExtentInverse);

			half4 SceneColorR = SampleSceneColor(CAUV.xy + SceneUVJitter.xy, ArrayIndex);
			half SceneColorG = SampleSceneColor(CAUV.zw + SceneUVJitter.xy, ArrayIndex).g;
			half SceneColorB = SampleSceneColor(SceneUV, ArrayIndex).b;
			SceneColor = SceneColorR;
			SceneColor.g = SceneColorG;
			SceneColor.b = SceneColorB;
		#else
			SceneColor = SampleSceneColor(SceneUV, ArrayIndex);
		#endif

		#if METAL_MSAA_HDR_DECODE && !USE_GAMMA_ONLY
			// look for PreTonemapMSAA 
			SceneColor.rgb *= rcp(SceneColor.r * (-0.299) + SceneColor.g * (-0.587) + SceneColor.b * (-0.114) + 1.0);
			// Try to kill negatives and NaNs here
			SceneColor.rgb = max(SceneColor.rgb, 0);
		#endif
	}

	// Sample bloom & mobile's vignette.
	#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
		float MobileVignette;
	#else
		const float MobileVignette = 1.0;
	#endif
	
	half3 Bloom;
	{
		float2 BloomUV;
		#if ES3_1_PROFILE
		{
			BloomUV = FullViewUV.xy;
		}
		#else
		{
			BloomUV = ApplyScreenTransform(UV.xy, ColorToBloom);
			BloomUV = clamp(BloomUV, BloomUVViewportBilinearMin, BloomUVViewportBilinearMax);
		}
		#endif

		float4 RawBloom;
		#if MOBILE_MULTI_VIEW
			RawBloom = Texture2DArraySample(BloomTexture, BloomSampler, float3(BloomUV,ArrayIndex));
		#else
			RawBloom = Texture2DSample(BloomTexture, BloomSampler, BloomUV);
		#endif

		#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
		{
			// Support sunshaft and vignette for mobile, and we have applied the BloomIntensity and the BloomDirtMask at the sun merge pass.
			Bloom = RawBloom.rgb;
			MobileVignette = RawBloom.a;
		}
		#else
		{
			float2 DirtLensUV = ConvertScreenViewportSpaceToLensViewportSpace(ScreenPos) * float2(1.0f, -1.0f);

			float3 RawBloomDirt = Texture2DSample(BloomDirtMaskTexture, BloomDirtMaskSampler, DirtLensUV * .5f + .5f).rgb;

			Bloom = RawBloom.rgb * (1.0 + RawBloomDirt * BloomDirtMaskTint.rgb);
		}
		#endif
	}
	
	// Compute scene color tint.
	#if USE_GAMMA_ONLY
		const float3 SceneColorTint = 1.0;
	#elif SUPPORTS_SCENE_COLOR_APPLY_PARAMETERS
		// Note that SceneColorApplyParamaters[0] is only for FFT bloom energy conservativeness = doesn't need to be in PreExposure
		const float3 SceneColorTint = ColorScale0.rgb * SceneColorApplyParamaters[0].xyz;
	#else
		const float3 SceneColorTint = ColorScale0.rgb;
	#endif

	// Compute the local exposure based of the scene color.
	#if LOCAL_EXPOSURE_MODE == LOCAL_EXPOSURE_MODE_BILATERAL
	float LocalExposure = 1.0;
	{
		float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor.rgb * OneOverPreExposure);
		float LogLuminance = log2(LuminanceVal);

		// Middle grey lum value adjusted by exposure compensation
		float MiddleGreyLumValue = log2(0.18 * EyeAdaptationData.y * LocalExposure_MiddleGreyExposureCompensation);

		float BaseLogLum = CalculateBaseLogLuminance(LogLuminance, LocalExposure_BlurredLuminanceBlend, GlobalExposure, FullViewUV, LumBilateralGrid, BlurredLogLum, LumBilateralGridSampler, BlurredLogLumSampler);
		LocalExposure = CalculateLocalExposure(LogLuminance + log2(GlobalExposure), BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, LocalExposure_DetailStrength);
	}
	#elif LOCAL_EXPOSURE_MODE == LOCAL_EXPOSURE_MODE_FUSION
	float LocalExposure = 1.0;
	{
		float2 ExposureFusionUV = ApplyScreenTransform(UV.xy, ColorToExposureFusion);
		ExposureFusionUV = clamp(ExposureFusionUV, ExposureFusion_UVViewportBilinearMin, ExposureFusion_UVViewportBilinearMax);

		const float FusionVal = Texture2DSample(ExposureFusionTexture, ExposureFusionSampler, ExposureFusionUV).x;

		const float LuminanceVal = CalculateEyeAdaptationLuminance(SceneColor.rgb * OneOverPreExposure * GlobalExposure);
		
		LocalExposure = CalculateFusionLocalExposure(LuminanceVal, FusionVal);
	}
	#else // LOCAL_EXPOSURE_MODE_NONE
	const float LocalExposure = 1.0;
	#endif

	// Apply sharpening onto SceneColor.
	#if USE_SHARPEN
	{
		// 0..x, 0:no effect .. 1:strong, from r.Tonemapper.Sharpen
		// Div6 is to save one instruction
		float SharpenMultiplierDiv6 = SharpenDiv6;

		half A0 = Luminance(SceneColor.rgb);

	#if HAS_PIXEL_QUAD_MESSAGE_PASSING_SUPPORT
		// Use Wave Intrinsics to reduce texture taps
		FPQMPContext PQMP = PQMPInit(floor(SceneUV* Color_Extent));
				
		half4 LuminanceNeighbors;
		
		half3 C1 = LookupSceneColor(SceneUV, ArrayIndex, float2(PQMP.QuadVector.x, 0)).rgb;
		half3 C3 = LookupSceneColor(SceneUV, ArrayIndex, float2(0, PQMP.QuadVector.y)).rgb;
		LuminanceNeighbors.x = Luminance(C1);
		LuminanceNeighbors.y = Luminance(C3);
		
		half3 C2 = PQMPReadX(PQMP, SceneColor.rgb);
		half3 C4 = PQMPReadY(PQMP, SceneColor.rgb);
		LuminanceNeighbors.z =  PQMPReadX(PQMP,A0);
		LuminanceNeighbors.w =  PQMPReadY(PQMP,A0);
	#else
		half3 C1 = LookupSceneColor(SceneUV, ArrayIndex, float2(-1,  0));
		half3 C2 = LookupSceneColor(SceneUV, ArrayIndex, float2( 1,  0));
		half3 C3 = LookupSceneColor(SceneUV, ArrayIndex, float2( 0, -1));
		half3 C4 = LookupSceneColor(SceneUV, ArrayIndex, float2( 0,  1));
		half4 LuminanceNeighbors = half4(Luminance(C1), Luminance(C2), Luminance(C3), Luminance(C4));
	#endif

		// compute mask to not sharpen near very bright HDR content
		// Note: using max instead of summming up saves 1 instruction
		// Note: We could multiply this to tweak the edge weight but it might instroduce a cost
		half HDREdge = (OneOverPreExposure * LocalExposure * GlobalExposure) * max4(abs(A0 - LuminanceNeighbors));
	
		// 0..1
		half EdgeMask = saturate(1.0f - HDREdge);

		// -1:sharpen, 0:no effect, 1:blur
		float LerpFactor = -EdgeMask * SharpenMultiplierDiv6;

		// optimized, Div6 went into the C++ code
		half3 DeltaColor = (C1 + C2 + C3 + C4) - SceneColor.rgb * 4;

		SceneColor.rgb += DeltaColor * LerpFactor;
	}
	#endif // USE_SHARPEN

#if FEATURE_LEVEL == FEATURE_LEVEL_ES3_1
	// Mobile device composers expect 1.0 in alpha
	float4 OutColor = float4(0.0, 0.0, 0.0, 1.0);
#else
	float4 OutColor = 0;
#endif

#if USE_GAMMA_ONLY
	{
		#if LOCAL_EXPOSURE_MODE != LOCAL_EXPOSURE_MODE_NONE
			#error Local exposure is not supported with gamma only
		#endif
		#if USE_SHARPEN
			#error Sharpen is not supported with gamma only
		#endif

		OutColor.rgb = pow(SceneColor.rgb * (OneOverPreExposure * GlobalExposure), InverseGamma.x);

		OutLuminance = Luminance(OutColor.rgb);
	}
#else // !USE_GAMMA_ONLY
	{
		// Compute vignette.
		half3 VignetteMask = MobileVignette;
		
		VignetteMask *= ComputeVignetteMask(
			Vignette, VignetteType, VignetteParams0, VignetteParams1, VignetteParams2
#if USE_VIGNETTE_TEXTURE
			, VignetteTexture, VignetteTextureSampler
#endif
		);
		
		// Compute the final linear color space with explicitly all the terms that should affect the pre exposure.
		half3 FinalLinearColor = SceneColor.rgb * SceneColorTint * (OneOverPreExposure * GlobalExposure * VignetteMask * LocalExposure);
		
		// Apply bloom.
		FinalLinearColor += Bloom * (OneOverPreExposure * GlobalExposure * VignetteMask);
		
		// Film grain is energy conservative so doesn't need to be taken into PreExposure
		#if USE_FILM_GRAIN
		if (IsFilmGrainEnabled())
		{
			float3 FilmGrainDecodeMultiply = FilmGrainTextureConstants[0].rgb;

			half FilmGrainIntensity = ComputeFilmGrainIntensity(FinalLinearColor);

			float2 FilmGrainUV = ApplyScreenTransform(ScreenPos, ScreenPosToFilmGrainTextureUV);

			half3 RawGrain = FilmGrainTexture.SampleLevel(FilmGrainSampler, FilmGrainUV, 0);
			half3 FinalGrain = RawGrain * half3(FilmGrainDecodeMultiply);

			FinalLinearColor.rgb *= lerp(1.0, FinalGrain, FilmGrainIntensity);
		}
		#endif
		
		half3 OutDeviceColor = ColorLookupTable(FinalLinearColor);

		half LuminanceForPostProcessAA = dot(OutDeviceColor, half3(0.299f, 0.587f, 0.114f));
		OutLuminance = LuminanceForPostProcessAA;

		// RETURN_COLOR not needed unless writing to SceneColor
		OutColor.rgb = OutDeviceColor;

		BRANCH
		if(GetOutputDevice() == TONEMAPPER_OUTPUT_LinearEXR)
		{
			OutColor.xyz = ST2084ToLinear(OutColor.xyz);
		}
		else
		{
			OutColor.xyz = OutDeviceColor;
		}
	}
#endif // !USE_GAMMA_ONLY
	
	#if !USE_GAMMA_ONLY
	{
		// Compute uniform grain on [0;1]
		half UniformGrainOP = GrainFromUV(GrainUV.zw); //TODO: verify the quality of GrainFromUV()

		// Compute uniform grain on [-1;1]
		half UniformGrainNP = UniformGrainOP * 2.0 - 1.0;
		half UniformGrainNPSign = clamp(UniformGrainNP * POSITIVE_INFINITY, half(-1.0), half(1.0));
		
		// Compute triangular grain on [-1;1]
		// half TriangularGrainOP = UniformGrainNPSign * (1.0 - sqrt(1.0 - abs(UniformGrainNP)));
		half TriangularGrainOP = UniformGrainNPSign - UniformGrainNPSign * sqrt(saturate(half(1.0) - abs(UniformGrainNP)));
		
		// Selects the grain to use.
		half FinalGrain = 0.0;
		#if DEBUG_GRAIN_QUANTIZATION
			if (ScreenPos.y > 1.0 / 3.0)
				FinalGrain = TriangularGrainOP;
			else if (ScreenPos.y < -1.0 / 3.0)
				FinalGrain = UniformGrainOP - 0.5;
		#elif 0
			FinalGrain = UniformGrainOP - 0.5;
		#else
			FinalGrain = TriangularGrainOP;
		#endif

		// Debug grain quantization.
		#if DEBUG_GRAIN_QUANTIZATION
		{
			half Quantization = 10.0;
			
			OutColor.rgb = round(OutColor.rgb * Quantization + FinalGrain) * rcp(Quantization);
			OutColor.rgb = OutColor.ggg;
		}
		#else
			OutColor.rgb += FinalGrain * BackbufferQuantizationDithering;
		#endif
	}
	#endif

	if ((Flags & FLAG_ALPHA_CHANNEL) != 0)
	{
		OutColor.a = SceneColor.a;
	}

	return OutColor;
}

// pixel shader entry point
void MainPS(
	in noperspective float2 UV : TEXCOORD0,
	in noperspective float2 InVignette : TEXCOORD1,
	in noperspective float4 GrainUV : TEXCOORD2,
	in noperspective float2 ScreenPos : TEXCOORD3,
	in noperspective float2 FullViewUV : TEXCOORD4, 
	in FStereoPSInput StereoInput,
	float4 SvPosition : SV_POSITION,		// after all interpolators
	out float4 OutColor : SV_Target0
#if OUTPUT_LUMINANCE
	, out float OutLuminance: SV_Target1
#endif
	)
{
	float Luminance;
	float3 UVI = float3(UV.xy, GetEyeIndex(StereoInput));
	OutColor = TonemapCommonPS(UVI, InVignette, GrainUV, ScreenPos, FullViewUV, SvPosition, Luminance);
#if OUTPUT_LUMINANCE
	OutLuminance = Luminance;
#endif
}

#if COMPUTESHADER
RWTexture2D<float4> RWOutputTexture;
RWTexture2D<float> RWOutputLuminance;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	float4 SvPosition = float4((float2)DispatchThreadId + 0.5f + Output_ViewportMin, 0.0f, 1.0f);
	float2 UV = ((float2)DispatchThreadId + 0.5f + Color_ViewportMin) * Color_ExtentInverse;

	if (IsComputeUVOutOfBounds(UV))
	{
		return;
	}

	float4 InScreenPos = float4(UV * float2(2, -2) - float2(1, -1), 0, 1);

	float2 Vignette;
	float4 GrainUV;
	float2 FullViewUV, ScreenPos;	
	TonemapCommonVS(InScreenPos, UV, Vignette, GrainUV, ScreenPos, FullViewUV);

	float Luminance;
	float4 OutColor = TonemapCommonPS(float3(UV.xy,0), Vignette, GrainUV, ScreenPos, FullViewUV, SvPosition, Luminance);

	uint2 PixelPos = DispatchThreadId + Output_ViewportMin;

	RWOutputTexture[PixelPos] = OutColor;
#if OUTPUT_LUMINANCE
	RWOutputLuminance[PixelPos] = Luminance;
#endif
}
#endif

#ifdef SUBPASS_MSAA_SAMPLES
	#if VULKAN_PROFILE
		#include "/Engine/Public/Platform/Vulkan/VulkanSubpassSupport.ush"
	#endif
#endif

half4 FetchAndResolveSceneColor(float2 UV, float ArrayIndex)
{
	half4 SceneColor = half4(1.0, 1.0, 1.0, 1.0);

#if VULKAN_PROFILE && SUBPASS_MSAA_SAMPLES > 0	
	#if SUBPASS_MSAA_SAMPLES == 8
		SceneColor = half4(VulkanSubpassFetch0(0) + VulkanSubpassFetch0(1) + VulkanSubpassFetch0(2) + VulkanSubpassFetch0(3) +
			VulkanSubpassFetch0(4) + VulkanSubpassFetch0(5) + VulkanSubpassFetch0(6) + VulkanSubpassFetch0(7)) * 0.125;
	#elif SUBPASS_MSAA_SAMPLES == 4
		SceneColor = half4(VulkanSubpassFetch0(0) + VulkanSubpassFetch0(1) + VulkanSubpassFetch0(2) + VulkanSubpassFetch0(3)) * 0.25;
	#elif SUBPASS_MSAA_SAMPLES == 2
		SceneColor = half4(VulkanSubpassFetch0(0) + VulkanSubpassFetch0(1)) * 0.5;
	#elif SUBPASS_MSAA_SAMPLES == 1
		SceneColor = half4(VulkanSubpassFetch0());
	#endif
#else
	#if MOBILE_MULTI_VIEW
		SceneColor = Texture2DArraySample(ColorTexture, ColorSampler, float3(UV.xy,ArrayIndex));
	#else
		SceneColor = Texture2DSample(ColorTexture, ColorSampler, UV);
	#endif
#endif
	return SceneColor;
}

void MobileCustomResolve_MainPS(
	in float4 SvPosition							: SV_POSITION, 
	in noperspective float2 UV						: TEXCOORD0,
	out				 float4 OutColor				: SV_Target0,
	in FStereoPSInput StereoInput
)
{
	StereoSetupPS(StereoInput);
	
	const float OneOverPreExposure = ResolvedView.OneOverPreExposure;

	half4 SceneColor = FetchAndResolveSceneColor(UV, GetEyeIndex(StereoInput));
	SceneColor.a = 1.0;

	half3 LinearColor = SceneColor.rgb * OneOverPreExposure;
	LinearColor *= ColorScale0.rgb; // SceneColorTint
	half3 DeviceColor = ColorLookupTable(LinearColor);

	OutColor.rgb = DeviceColor;
	OutColor.a = 1.0;
}
