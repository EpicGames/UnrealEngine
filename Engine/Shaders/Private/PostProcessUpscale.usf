// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"
#include "PaniniProjection.ush"
#include "TextureSampling.ush"
#include "LensDistortion.ush"
#include "PixelQuadMessagePassing.ush"

#define UPSCALE_METHOD_NEAREST 1
#define UPSCALE_METHOD_BILINEAR 2
#define UPSCALE_METHOD_DIRECTIONAL 3
#define UPSCALE_METHOD_CATMULL_ROM 4
#define UPSCALE_METHOD_LANCZOS 5
#define UPSCALE_METHOD_GAUSSIAN 6
#define UPSCALE_METHOD_SMOOTHSTEP 7
#define UPSCALE_METHOD_AREA 8

#define METHOD_SET_SIMPLE 1
#define METHOD_SET_COMPLEX 2

#ifndef DIM_METHOD
#define DIM_METHOD 0
#endif

#if DIM_METHOD == UPSCALE_METHOD_NEAREST
	#define INCLUDE_UPSCALE_METHOD_NEAREST 1
#elif DIM_METHOD == UPSCALE_METHOD_BILINEAR
	#define INCLUDE_UPSCALE_METHOD_BILINEAR 1
#elif DIM_METHOD == UPSCALE_METHOD_DIRECTIONAL
	#define INCLUDE_UPSCALE_METHOD_DIRECTIONAL 1
#elif DIM_METHOD == UPSCALE_METHOD_CATMULL_ROM
	#define INCLUDE_UPSCALE_METHOD_CATMULL_ROM 1
#elif DIM_METHOD == UPSCALE_METHOD_LANCZOS
	#define INCLUDE_UPSCALE_METHOD_LANCZOS 1
#elif DIM_METHOD == UPSCALE_METHOD_GAUSSIAN
	#define INCLUDE_UPSCALE_METHOD_GAUSSIAN 1
#elif DIM_METHOD == UPSCALE_METHOD_SMOOTHSTEP
	#define INCLUDE_UPSCALE_METHOD_SMOOTHSTEP 1
#elif DIM_METHOD == UPSCALE_METHOD_AREA
	#define INCLUDE_UPSCALE_METHOD_AREA 1
// see GetMethodSet
#elif DIM_METHODSET == METHOD_SET_SIMPLE
	#define INCLUDE_UPSCALE_METHOD_NEAREST 1
	#define INCLUDE_UPSCALE_METHOD_BILINEAR 1
	#define INCLUDE_UPSCALE_METHOD_SMOOTHSTEP 1
	#define INCLUDE_UPSCALE_METHOD_AREA 1
#elif DIM_METHODSET == METHOD_SET_COMPLEX
	#define INCLUDE_UPSCALE_METHOD_DIRECTIONAL 1
	#define INCLUDE_UPSCALE_METHOD_CATMULL_ROM 1
	#define INCLUDE_UPSCALE_METHOD_LANCZOS 1
	#define INCLUDE_UPSCALE_METHOD_GAUSSIAN 1
#endif

float UpscaleSoftness;

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D<float2> DistortingDisplacementTexture;
SamplerState DistortingDisplacementSampler;
Texture2D<float2> UndistortingDisplacementTexture;
SamplerState UndistortingDisplacementSampler;

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

// Point-sampled version (used on mobile). Needs to be a separate texture because of OpenGL fused samplers.
Texture2D PointSceneColorTexture;
SamplerState PointSceneColorSampler;

//in a multiview case, PointSceneColorTexture is an array texture
Texture2DArray PointSceneColorTextureArray;

uint2 GridDimensions;
uint bInvertAlpha;

float Sharpening;

// Do a smoothstep(x) = 3 x^2 - 2 x^3
float2 GetSmoothstepUV(float2 LinearUV, float2 TextureSize, float2 TextureInvSize)
{
	// Top left cornered pixel coordinate to sample.
	float2 PixelCoord = LinearUV * TextureSize - 0.5;

	// Index of the top left pixel used in the bilinear interpolation.
	float2 TopLeftPixelCoord = floor(PixelCoord);

	// Interpolation factors in the 2x2 quad.
	float2 PixelInterp = PixelCoord - TopLeftPixelCoord;
	
	// New interpolation factors in the 2x2 quad with smoothstep.
	float2 SmoothPixelInterp = PixelInterp * PixelInterp * (3 - 2 * PixelInterp);

	// Returns new UV coordinate.
	return TextureInvSize * (TopLeftPixelCoord + SmoothPixelInterp + 0.5);
}

float2 GetAreaUV(float2 LinearCoord, float2 TextureSize, float2 TextureInvSize)
{
	float2 MinCoord = mad(-0.5, Output_ViewportSizeInverse, LinearCoord);
	float2 MaxCoord = MinCoord + Output_ViewportSizeInverse;
	float2 FirstIndex = floor(MinCoord * TextureSize);
	float2 LastIndex = ceil(MaxCoord * TextureSize) - 1.;
	float2 LastCoord = LastIndex * TextureInvSize;
	float2 bSameTexel = step(LastIndex, FirstIndex);
	float2 AdjustmentInTexels = lerp(
		(LinearCoord - LastCoord) * Output_ViewportSize, // Fair filter
		.5, // Sample center
		bSameTexel);
	return mad(TextureInvSize, AdjustmentInTexels, LastCoord);
}

float Luma(float3 Color) 
{
#if UE_LEGACY_LUMINANCE_FACTORS || WORKING_COLOR_SPACE_IS_SRGB
	// Note: In this case, the previous (legacy) luminance factors correctly used the sRGB definition.
	float3 LuminanceFactors = float3(0.2126390059, 0.7151686788, 0.0721923154);
#else
	float3 LuminanceFactors = float3(WORKING_COLOR_SPACE_RGB_TO_XYZ_MAT._m10_m11_m12);
#endif

	return dot(Color, LuminanceFactors);
}

float3 RGBToYCoCg( float3 RGB )
{
	float Y  = dot( RGB, float3(  1, 2,  1 ) );
	float Co = dot( RGB, float3(  2, 0, -2 ) );
	float Cg = dot( RGB, float3( -1, 2, -1 ) );
	
	float3 YCoCg = float3( Y, Co, Cg );
	return YCoCg;
}

float3 YCoCgToRGB( float3 YCoCg )
{
	float Y  = YCoCg.x * 0.25;
	float Co = YCoCg.y * 0.25;
	float Cg = YCoCg.z * 0.25;

	float R = Y + Co - Cg;
	float G = Y + Cg;
	float B = Y - Co - Cg;

	float3 RGB = float3( R, G, B );
	return RGB;
}

float Gaussian(float Scale, float2 Offset) 
{
	return exp2(Scale * dot(Offset, Offset));
}

float4 SampleSceneColorRGBA(float2 BufferUV)
{
	BufferUV = clamp(BufferUV, Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);

	return SceneColorTexture.SampleLevel(SceneColorSampler, BufferUV, 0).rgba;
}

float4 AccumulateAndApplyWeight(in float4 DataRGBA, in float Weight, inout float WeightsSum)
{
	WeightsSum += Weight;
	return DataRGBA * Weight;
}



float4 Upsample(float2 InputUV, uint EyeIndex, uint Method)
{
	float4 OutColor = 0;

	switch(Method)
	{
#if INCLUDE_UPSCALE_METHOD_NEAREST
	case UPSCALE_METHOD_NEAREST:
	{
	// Nearest sampling (not blurry but blocky, more for testing)
	#if ES3_1_PROFILE
		#if MOBILE_MULTI_VIEW
			OutColor = Texture2DArraySample(PointSceneColorTextureArray, PointSceneColorSampler, float3(InputUV,EyeIndex));
		#else
			OutColor = Texture2DSample(PointSceneColorTexture, PointSceneColorSampler, InputUV);
		#endif
	#else
		#if MOBILE_MULTI_VIEW
			OutColor = PointSceneColorTextureArray.SampleLevel(PointSceneColorSampler, vec3(InputUV,EyeIndex), 0, int2(0, 0));
		#else
			OutColor = PointSceneColorTexture.SampleLevel(PointSceneColorSampler, InputUV, 0, int2(0, 0));
		#endif
	#endif
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_BILINEAR
	case UPSCALE_METHOD_BILINEAR:
	{
		// Bilinear (fast, aliasing)
		OutColor = SampleSceneColorRGBA(InputUV);
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_DIRECTIONAL
	case UPSCALE_METHOD_DIRECTIONAL:
	{
		// Directional blur with unsharp mask upsample.
		float2 UV = InputUV;
		float X = 0.5;
		float4 ColorNW = SampleSceneColorRGBA(UV + float2(-X, -X) * Input_ExtentInverse);
		float4 ColorNE = SampleSceneColorRGBA(UV + float2( X, -X) * Input_ExtentInverse);
		float4 ColorSW = SampleSceneColorRGBA(UV + float2(-X,  X) * Input_ExtentInverse);
		float4 ColorSE = SampleSceneColorRGBA(UV + float2( X,  X) * Input_ExtentInverse);
		OutColor = (ColorNW * 0.25) + (ColorNE * 0.25) + (ColorSW * 0.25) + (ColorSE * 0.25);
	
		float LumaNW = Luma(ColorNW.rgb);
		float LumaNE = Luma(ColorNE.rgb);
		float LumaSW = Luma(ColorSW.rgb);
		float LumaSE = Luma(ColorSE.rgb);

		float2 IsoBrightnessDir;
		float DirSWMinusNE = LumaSW - LumaNE;
		float DirSEMinusNW = LumaSE - LumaNW;
		IsoBrightnessDir.x = DirSWMinusNE + DirSEMinusNW;
		IsoBrightnessDir.y = DirSWMinusNE - DirSEMinusNW;

		// avoid NaN on zero vectors by adding 2^-24 (float ulp when length==1, and also minimum representable half)
		IsoBrightnessDir = IsoBrightnessDir * (0.125 * rsqrt(dot(IsoBrightnessDir, IsoBrightnessDir) + 6e-8));

		float4 ColorN = SampleSceneColorRGBA(UV - IsoBrightnessDir * Input_ExtentInverse);
		float4 ColorP = SampleSceneColorRGBA(UV + IsoBrightnessDir * Input_ExtentInverse);

		float UnsharpMask = 0.25;
		OutColor = (ColorN + ColorP) * ((UnsharpMask + 1.0) * 0.5) - (OutColor * UnsharpMask);
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_CATMULL_ROM
	case UPSCALE_METHOD_CATMULL_ROM:
	{
		// Bicubic Catmull-Rom in five samples
		FCatmullRomSamples Samples = GetBicubic2DCatmullRomSamples(InputUV, Input_Extent, Input_ExtentInverse);
		for (uint i = 0; i < Samples.Count; i++)
		{
			OutColor += SampleSceneColorRGBA(Samples.UV[i]) * Samples.Weight[i];
		}
		OutColor *= Samples.FinalMultiplier;
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_LANCZOS
	case UPSCALE_METHOD_LANCZOS:
	{
		// Lanczos 3
		float2 UV = InputUV * Input_Extent;
		float2 tc = floor(UV - 0.5) + 0.5;
		float2 f = UV - tc + 2;

		// compute at f, f-1, f-2, f-3, f-4, and f-5 using trig angle addition
		float2 fpi = f*PI, fpi3 = f * (PI / 3.0);
		float2 sinfpi = sin(fpi), sinfpi3 = sin(fpi3), cosfpi3 = cos(fpi3);
		const float r3 = sqrt(3.0);
		float2 w0 = ( sinfpi *       sinfpi3              ) / ( f       * f       );
		float2 w1 = (-sinfpi * (     sinfpi3 - r3*cosfpi3)) / ((f - 1.0)*(f - 1.0));
		float2 w2 = ( sinfpi * (    -sinfpi3 - r3*cosfpi3)) / ((f - 2.0)*(f - 2.0));
		float2 w3 = (-sinfpi * (-2.0*sinfpi3             )) / ((f - 3.0)*(f - 3.0));
		float2 w4 = ( sinfpi * (    -sinfpi3 + r3*cosfpi3)) / ((f - 4.0)*(f - 4.0));
		float2 w5 = (-sinfpi * (     sinfpi3 + r3*cosfpi3)) / ((f - 5.0)*(f - 5.0));

		// use bilinear texture weights to merge center two samples in each dimension
		float2 Weight[5];
		Weight[0] = w0;
		Weight[1] = w1;
		Weight[2] = w2 + w3;
		Weight[3] = w4;
		Weight[4] = w5;

		float2 Sample[5];
		Sample[0] = Input_ExtentInverse * (tc - 2);
		Sample[1] = Input_ExtentInverse * (tc - 1);
		Sample[2] = Input_ExtentInverse * (tc + w3 / Weight[2]);
		Sample[3] = Input_ExtentInverse * (tc + 2);
		Sample[4] = Input_ExtentInverse * (tc + 3);

		OutColor = 0;
		float WeightsSum = 0;

		// 5x5 footprint with corners dropped to give 13 texture taps
		OutColor += AccumulateAndApplyWeight(SampleSceneColorRGBA(float2(Sample[0].x, Sample[2].y)), Weight[0].x * Weight[2].y, WeightsSum);

		OutColor += AccumulateAndApplyWeight(SampleSceneColorRGBA(float2(Sample[1].x, Sample[1].y)), Weight[1].x * Weight[1].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight(SampleSceneColorRGBA(float2(Sample[1].x, Sample[2].y)), Weight[1].x * Weight[2].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight(SampleSceneColorRGBA(float2(Sample[1].x, Sample[3].y)), Weight[1].x * Weight[3].y, WeightsSum);

		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[2].x, Sample[0].y)), Weight[2].x * Weight[0].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[2].x, Sample[1].y)), Weight[2].x * Weight[1].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[2].x, Sample[2].y)), Weight[2].x * Weight[2].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[2].x, Sample[3].y)), Weight[2].x * Weight[3].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[2].x, Sample[4].y)), Weight[2].x * Weight[4].y, WeightsSum);

		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[3].x, Sample[1].y)), Weight[3].x * Weight[1].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[3].x, Sample[2].y)), Weight[3].x * Weight[2].y, WeightsSum);
		OutColor += AccumulateAndApplyWeight( SampleSceneColorRGBA(float2(Sample[3].x, Sample[3].y)), Weight[3].x * Weight[3].y, WeightsSum);

		OutColor += AccumulateAndApplyWeight(SampleSceneColorRGBA(float2(Sample[4].x, Sample[2].y)), Weight[4].x * Weight[2].y, WeightsSum);

		OutColor /= WeightsSum;
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_GAUSSIAN
	case UPSCALE_METHOD_GAUSSIAN:
	{
		// Gaussian filtered unsharp mask
		float2 UV = InputUV * Input_Extent;
		float2 tc = floor(UV) + 0.5;

		// estimate pixel value and derivatives
		OutColor = 0;
		float4 Laplacian = 0;
		float WeightsSum = 0;
		UNROLL for (int i = -3; i <= 2; ++i)
		{
			UNROLL for (int j = -3; j <= 2; ++j)
			{
				float2 TexelOffset = float2(i, j) + 0.5;

				// skip corners: eliminated entirely by UNROLL
				if (dot(TexelOffset, TexelOffset) > 9) continue;

				float2 Texel = tc + TexelOffset;
				float2 Offset = UV - Texel;
				float OffsetSq = 2 * dot(Offset, Offset);	// texel loop is optimized for variance = 0.5
				float Weight = exp(-0.5 * OffsetSq);
				float4 Sample = AccumulateAndApplyWeight(SampleSceneColorRGBA(Texel * Input_ExtentInverse), Weight, WeightsSum);

				OutColor += Sample;
				Laplacian += Sample * (OffsetSq - 2);
			}
		}

		const float InvWeightsSum = 1.0f / WeightsSum;
		OutColor  *= InvWeightsSum;
		Laplacian *= InvWeightsSum;

		float UnsharpScale = UpscaleSoftness * (1 - Input_Extent.x * Input_Extent.y * Output_ViewportSizeInverse.x * Output_ViewportSizeInverse.y);
		OutColor -= UnsharpScale * Laplacian;
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_SMOOTHSTEP
	case UPSCALE_METHOD_SMOOTHSTEP:
	{
		OutColor = SampleSceneColorRGBA(GetSmoothstepUV(InputUV, Input_Extent, Input_ExtentInverse));
	}
	break;
#endif
#if INCLUDE_UPSCALE_METHOD_AREA
	case UPSCALE_METHOD_AREA:
	{
		OutColor = SampleSceneColorRGBA(GetAreaUV(InputUV, Input_Extent, Input_ExtentInverse));
	}
	break;
#endif
	}

	return OutColor;
}

float4 ProcessAlpha(float4 InColor)
{
	float4 OutColor = InColor;

#if defined(DIM_ALPHA_CHANNEL) && DIM_ALPHA_CHANNEL == 1
	FLATTEN
	if (OutColor.a > 0.999)
	{
		OutColor.a = 1;
	}
	else if (OutColor.a < 0.001)
	{
		OutColor.a = 0.0;
	}

	OutColor.a = select(bInvertAlpha > 0, 1.0f - OutColor.a, OutColor.a);
	
#else
	OutColor.a = 0; // Skip all computations related to alpha
#endif

	return OutColor;
}

#if COMPUTESHADER

RWTexture2D<float4> RWOutputTexture;

groupshared uint LDS[(THREADGROUP_SIZEX) * (THREADGROUP_SIZEY)];

uint LDSGetIndex(uint2 GroupThreadId)
{
	return (GroupThreadId.y * THREADGROUP_SIZEX) + GroupThreadId.x;
}

void LDSWriteFloat(uint2 GroupThreadId, float LocalValue)
{
	LDS[LDSGetIndex(GroupThreadId)] = asuint(LocalValue);
}

float LDSReadFloatAtIndex(uint Index)
{
	return asfloat(LDS[Index]);
}

float LDSReadFloat(uint2 GroupThreadId, int2 Direction, out bool bOutOfBounds)
{
	int2 ReadPosition = GroupThreadId + Direction;
	int2 ClampedReadPosition = clamp(ReadPosition, int2(0, 0), int2(THREADGROUP_SIZEX-1, THREADGROUP_SIZEY-1));
	bOutOfBounds = any(ClampedReadPosition != ReadPosition);

	uint LDSValue = LDS[LDSGetIndex(ClampedReadPosition)];
	float Result = asfloat(LDSValue);

	return Result;
}

uint GetTotalNumBoundarySamples()
{
	return 2*(THREADGROUP_SIZEX + 2) + 2*(THREADGROUP_SIZEY);
}

int2 IndexToBoundaryPosition(int Index)
{
	// Generates positions on the boundary as shown below.
	//
	// e.g. for threadgroup size of 4x4:
	//        Y
	//        |
	//   E F G H I J
	//   D         0
	//   C         1
	//   B         2 -X
	//   A         3
	//   9 8 7 6 5 4
	int2 Result;
	if (Index < THREADGROUP_SIZEY)
	{
		Result.x = THREADGROUP_SIZEX;
		Result.y = Index;
	}
	else if (Index < THREADGROUP_SIZEY + THREADGROUP_SIZEX + 1)
	{
		Result.x = THREADGROUP_SIZEX - (Index - THREADGROUP_SIZEY);
		Result.y = THREADGROUP_SIZEY;
	}
	else if (Index < THREADGROUP_SIZEY + THREADGROUP_SIZEX + THREADGROUP_SIZEY + 2)
	{
		Result.x = -1;
		Result.y = THREADGROUP_SIZEY - (Index - (THREADGROUP_SIZEY + THREADGROUP_SIZEX + 1));
	}
	else
	{
		Result.x = -1 + (Index - (THREADGROUP_SIZEY + THREADGROUP_SIZEX + THREADGROUP_SIZEY + 2));
		Result.y = -1;
	}
	return Result;
}

uint BoundaryPositionToIndex(int2 Position)
{
	if (Position.x == THREADGROUP_SIZEX && Position.y >= 0)
	{
		return Position.y;
	}
	else if (Position.y == THREADGROUP_SIZEY && Position.x >= 0)
	{
		return THREADGROUP_SIZEY + THREADGROUP_SIZEX - Position.x;
	}
	else if (Position.x == -1 && Position.y >= 0)
	{
		return THREADGROUP_SIZEY + THREADGROUP_SIZEX + THREADGROUP_SIZEY + 1 - Position.y;
	}
	else// if (Position.y == -1)
	{
		return (Position.x + 1) + THREADGROUP_SIZEY + THREADGROUP_SIZEX + THREADGROUP_SIZEY + 2;
	}
}

float2 ApplyLensDistortion(float2 UV, uint2 GroupId, float2 PixelGroupUV)
{
	float2 DistortedUV = UV;

#if DIM_LENS_DISTORTION

#if 1 //subsample distortion and interpolate
	const uint2 GroupSize = uint2(THREADGROUP_SIZEX, THREADGROUP_SIZEY);
	float2 GroupSizeUV = GroupSize * Output_ViewportSizeInverse;

	uint2 TopLeftPixelPosition = GroupId * GroupSize;
	float2 TopLeftUV     = ((float2)TopLeftPixelPosition + 0.5f) * Output_ViewportSizeInverse;
	float2 TopRightUV    = TopLeftUV + float2(GroupSizeUV.x, 0);
	float2 BottomLeftUV  = TopLeftUV + float2(0, GroupSizeUV.y);
	float2 BottomRightUV = TopLeftUV + float2(GroupSizeUV.x, GroupSizeUV.y);

	float4x2 Samples;
	Samples[0] = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, TopLeftUV);
	Samples[1] = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, TopRightUV);
	Samples[2] = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, BottomLeftUV);
	Samples[3] = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, BottomRightUV);

	float4 BilinearSampleWeights;
	BilinearSampleWeights[0] = (1 - PixelGroupUV.x) * (1 - PixelGroupUV.y);
	BilinearSampleWeights[1] = (PixelGroupUV.x) * (1 - PixelGroupUV.y);
	BilinearSampleWeights[2] = (1 - PixelGroupUV.x) * (PixelGroupUV.y);
	BilinearSampleWeights[3] = (PixelGroupUV.x) * (PixelGroupUV.y);

	DistortedUV = mul(BilinearSampleWeights, Samples);
#else
	DistortedUV = ApplyLensDistortionOnViewportUV(UndistortingDisplacementTexture, UndistortingDisplacementSampler, UV);
#endif

#endif

	return DistortedUV;
}

/*half GrainFromUV(float2 GrainUV) 
{
	half Grain = frac(sin(GrainUV.x + GrainUV.y * 543.31) *  493013.0);
	return Grain;
}

float3 CalculateFilmGrain(float2 GrainUV)
{
	// Compute uniform grain on [0;1]
	half UniformGrainOP = GrainFromUV(GrainUV.zw); //TODO: verify the quality of GrainFromUV()

	// Compute uniform grain on [-1;1]
	half UniformGrainNP = UniformGrainOP * 2.0 - 1.0;
	half UniformGrainNPSign = clamp(UniformGrainNP * POSITIVE_INFINITY, half(-1.0), half(1.0));
		
	// Compute triangular grain on [-1;1]
	half TriangularGrainOP = UniformGrainNPSign - UniformGrainNPSign * sqrt(saturate(half(1.0) - abs(UniformGrainNP)));
		
	// Selects the grain to use.
	half FinalGrain = TriangularGrainOP;

	float BackbufferQuantizationDithering = 0;
	return FinalGrain * BackbufferQuantizationDithering;
}*/

float2 PixelPositionToInputUV(uint2 PixelPosition, uint2 GroupId, float2 PixelGroupUV)
{
	float2 OutputUV = ((float2)PixelPosition + 0.5f) * Output_ViewportSizeInverse;
	float2 DistortedOutputUV = ApplyLensDistortion(OutputUV, GroupId, PixelGroupUV);
	float2 InputBufferUV = (Input_ViewportMin + DistortedOutputUV * Input_ViewportSize) * Input_ExtentInverse;
	return InputBufferUV;
}

uint UpscaleMethod;

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint2 GroupId : SV_GroupID, uint2 GroupThreadId : SV_GroupThreadID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint2 GroupSize = uint2(THREADGROUP_SIZEX, THREADGROUP_SIZEY);

	const uint EyeIndex = 0; // Unsupported
	int2 PixelPosition = DispatchThreadId;
	int2 BufferPixelPosition = PixelPosition + Output_ViewportMin;
	float2 PixelGroupUV = float2(GroupThreadId) / GroupSize;

	float2 InputBufferUV = PixelPositionToInputUV(PixelPosition, GroupId, PixelGroupUV);

	float4 OutColor = Upsample(InputBufferUV, EyeIndex, UpscaleMethod);

#if DIM_SHARPENING_QUALITY > 0
	float3 ColorYCoCg = RGBToYCoCg(OutColor.rgb);
	float LocalLuminance = ColorYCoCg.x;

	LDSWriteFloat(GroupThreadId, LocalLuminance);
	GroupMemoryBarrierWithGroupSync();

	bool bOutOfBoundsLeft, bOutOfBoundsTop, bOutOfBoundsRight, bOutOfBoundsBottom;
	float LuminanceLeft   = LDSReadFloat(GroupThreadId, uint2(-1,  0), bOutOfBoundsLeft);
	float LuminanceTop    = LDSReadFloat(GroupThreadId, uint2( 0, -1), bOutOfBoundsTop);
	float LuminanceRight  = LDSReadFloat(GroupThreadId, uint2( 1,  0), bOutOfBoundsRight);
	float LuminanceBottom = LDSReadFloat(GroupThreadId, uint2( 0,  1), bOutOfBoundsBottom);

	bool bOutOfBoundsTopLeft, bOutOfBoundsTopRight, bOutOfBoundsBottomRight, bOutOfBoundsBottomLeft;
	float LuminanceTopLeft     = LDSReadFloat(GroupThreadId, uint2(-1, -1), bOutOfBoundsTopLeft);
	float LuminanceTopRight    = LDSReadFloat(GroupThreadId, uint2( 1, -1), bOutOfBoundsTopRight);
	float LuminanceBottomRight = LDSReadFloat(GroupThreadId, uint2( 1,  1), bOutOfBoundsBottomRight);
	float LuminanceBottomLeft  = LDSReadFloat(GroupThreadId, uint2(-1,  1), bOutOfBoundsBottomLeft);

#if DIM_SHARPENING_QUALITY > 1
	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadIndex < GetTotalNumBoundarySamples())
	{
		int2 TopLeftPixelPosition = GroupId * GroupSize;
		int2 BoundaryPixelPosRelativeToGroup = IndexToBoundaryPosition(GroupThreadIndex);
		float2 BoundaryPixelGroupUV = BoundaryPixelPosRelativeToGroup / GroupSize;
		int2 BoundaryPixelPosition = clamp(TopLeftPixelPosition + BoundaryPixelPosRelativeToGroup, int2(0,0), Output_ViewportSize-1);

		float2 BoundaryPixelInputUV = PixelPositionToInputUV(BoundaryPixelPosition, GroupId, BoundaryPixelGroupUV); // approximates distortion
		//float4 BoundaryPixel = Upsample(BoundaryPixelInputUV, EyeIndex, UpscaleMethod);
		float4 BoundaryPixel = SampleSceneColorRGBA(BoundaryPixelInputUV); // use simple bilinear instead of full upscaler for speed
		float3 BoundaryPixelYCoCg = RGBToYCoCg(BoundaryPixel.rgb);
		float BoundaryPixelLuminance = BoundaryPixelYCoCg.x;
		LDSWriteFloat(GroupThreadId, BoundaryPixelLuminance);
	}

	GroupMemoryBarrierWithGroupSync();

	if (bOutOfBoundsLeft)        { LuminanceLeft   = LDSReadFloatAtIndex(BoundaryPositionToIndex(GroupThreadId + uint2(-1,  0))); }
	else if (bOutOfBoundsRight)  { LuminanceRight  = LDSReadFloatAtIndex(BoundaryPositionToIndex(GroupThreadId + uint2( 1,  0))); }
	if (bOutOfBoundsTop)         { LuminanceTop    = LDSReadFloatAtIndex(BoundaryPositionToIndex(GroupThreadId + uint2( 0, -1))); }
	else if (bOutOfBoundsBottom) { LuminanceBottom = LDSReadFloatAtIndex(BoundaryPositionToIndex(GroupThreadId + uint2( 0,  1))); }
#endif

	float DeltaLuminance = ( -(LuminanceLeft + LuminanceTop + LuminanceRight + LuminanceBottom) + (LocalLuminance * 4.0) );
	LocalLuminance += DeltaLuminance * Sharpening;

	ColorYCoCg.x = LocalLuminance;
	OutColor.rgb = YCoCgToRGB(ColorYCoCg);
#endif

	OutColor = ProcessAlpha(OutColor);

	if (any(PixelPosition >= Output_ViewportSize))
	{
		return;
	}

	RWOutputTexture[BufferPixelPosition] = OutColor;
}

#else // !COMPUTESHADER

// vertex shader entry point
void MainVS(
	in uint VertexId : SV_VertexID,
	in FStereoVSInput StereoInput,
	out noperspective float4 OutTexCoord : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION)
{
	StereoSetupVS(StereoInput, StereoOutput);
	
	float2 TexCoord = float2(VertexId % (GridDimensions.x + 1), VertexId / (GridDimensions.x + 1)) / float2(GridDimensions);

	// still in 0..1 range
	float4 Position = float4(TexCoord.x, TexCoord.y, 0, 1);

	// distort pos
	Position.xy = ApplyLensDistortionOnViewportUV(DistortingDisplacementTexture, DistortingDisplacementSampler, TexCoord);

	DrawRectangle(Position, TexCoord, OutPosition, OutTexCoord.xy);
	OutTexCoord.zw =  OutPosition.xy;
}

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	StereoSetupPS(StereoInput);
	const uint EyeIndex = GetEyeIndex(StereoInput);

	OutColor = Upsample(UVAndScreenPos.xy, EyeIndex, DIM_METHOD);
	OutColor = ProcessAlpha(OutColor);
}

#endif // !COMPUTESHADER