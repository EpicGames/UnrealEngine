// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D InputTexture;
SamplerState InputSampler;
float4 SelectionColor;
float InvGamma;
float bShowTileBorder;

/** Main pixel shader function */
void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float Alpha = 1;

	int2 ViewportPixelPos = (int2)SvPosition.xy - Output_ViewportMin;

	const int Border1 = 10;
	const int Border2 = 8;
	float BorderDistance = ComputeDistanceToRect(ViewportPixelPos, int2(0, 0) + Border1.xx, Output_ViewportSize - 2 * Border1.xx);
	Alpha = lerp(1.0f, (1 - saturate((BorderDistance - Border2.x) * 0.5f)), bShowTileBorder);

	OutColor = float4(Texture2DSample(InputTexture, InputSampler, UVAndScreenPos.xy).rgb, Alpha);

	OutColor.rgb = pow(OutColor.rgb, InvGamma);

	// border around selected tile
	float BorderFactor = saturate(BorderDistance - Border2.x) * bShowTileBorder;
	OutColor.xyz = lerp(OutColor.xyz, SelectionColor.xyz, BorderFactor);
	OutColor.w = lerp(OutColor.w, SelectionColor.w, BorderFactor);
	OutColor.w = lerp(OutColor.w, 0.0f, saturate(BorderDistance - Border1.x) * bShowTileBorder);
}