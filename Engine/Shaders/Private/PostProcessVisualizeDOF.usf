// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "DepthOfFieldCommon.ush"

Texture2D ColorTexture;
SamplerState ColorSampler;

Texture2D DepthTexture;
SamplerState DepthSampler;

FScreenTransform SvPositionToColorUV;
FScreenTransform SvPositionToDepthUV;

float4 NearColor;
float4 FarColor;

// render in fullres to visualize the half res DOF input from the setup pass
void VisualizeDOFPS(
	in noperspective float4 UVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	OutColor = 0;

	float SceneDeviceZ = DepthTexture.SampleLevel(DepthSampler, ApplyScreenTransform(SvPosition.xy, SvPositionToDepthUV), 0).r;
	half3 SceneColor = Texture2DSample(ColorTexture, ColorSampler, ApplyScreenTransform(SvPosition.xy, SvPositionToColorUV)).rgb;

	float SceneDepth = ConvertFromDeviceZ(SceneDeviceZ);

	// 0..1
	half Near = CalcUnfocusedPercentCustomBound(SceneDepth, 1, 0);
	// 0..1
	half Far = CalcUnfocusedPercentCustomBound(SceneDepth, 0, 1);

	OutColor = 0;
	OutColor.rgb = lerp(OutColor.rgb, NearColor.rgb, Near);
	OutColor.rgb = lerp(OutColor.rgb, FarColor.rgb, Far);

	// blend in a bit of the scene color to make navigation easier
	OutColor.rgb = lerp(saturate(OutColor.rgb), Luminance(SceneColor), 0.1f);
}