// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"
#include "PostProcessHistogramCommon.ush"
#include "DeferredShadingCommon.ush"
#include "TonemapCommon.ush"
#include "MiniFontCommon.ush"
#include "ColorSpace.ush"

// the prior frame's eye adaptation settings
StructuredBuffer<float4> EyeAdaptationBuffer;

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

Texture2D HDRSceneColorTexture;
SamplerState HDRSceneColorSampler;

FScreenTransform ColorUVToLuminanceUV;
Texture2D LuminanceTexture;
SamplerState LuminanceSampler;

#if SHADING_PATH_MOBILE
Buffer<uint> HistogramBuffer;
#else
Texture2D HistogramTexture;
#endif

// Mobile reads Buffer<uint>, desktop reads Texture2D.
float GetHistogramBucketForVis(uint BucketIndex)
{
#if SHADING_PATH_MOBILE
	return GetHistogramBucket(HistogramBuffer, BucketIndex);
#else
	return GetHistogramBucket(HistogramTexture, BucketIndex);
#endif
}

float ComputeHistogramSumForVis()
{
#if SHADING_PATH_MOBILE
	return ComputeHistogramSum(HistogramBuffer);
#else
	return ComputeHistogramSum(HistogramTexture);
#endif
}

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

float IlluminanceMeterEnabled;
float UsingIlluminance;
float LuminanceScale;

// only needed for nice visualization
float ComputeHistogramMax()
{
	float Max = 0;

	for(uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		Max = max(Max, GetHistogramBucketForVis(i));
	}

	return Max;
}

float ComputePlaceInHistogram(float3 HDRColor)
{
	float LogLuminance = log2(dot(HDRColor,float3(1.0f,1.0f,1.0f))/3.0f);

	float SumTotal = 0.0;
	float SumBelow = 0.0f;

	for(uint i = 0; i < HISTOGRAM_SIZE; ++i)
	{
		float Weight = GetHistogramBucketForVis(i);

		float LogLuminanceLow = ComputeLogLuminanceFromHistogramPosition(float(i) / (float)(HISTOGRAM_SIZE-1));
		float LogLuminanceHigh = ComputeLogLuminanceFromHistogramPosition(float(i+1) / (float)(HISTOGRAM_SIZE-1));
		
		// if log luminace is greater than high, it's 1.0.
		// if log luminace is lower than low, it's 0.0
		// else it's a lerp between
		float T = saturate((LogLuminance - LogLuminanceLow)/(LogLuminanceHigh - LogLuminanceLow));

		SumBelow += Weight * T;
		SumTotal += Weight;
	}

	float Percentile = SumBelow/SumTotal;

	return Percentile;
}

bool InUnitBox(float2 UV)
{
	return UV.x >= 0 && UV.y >= 0 && UV.y < 1 && UV.y < 1;
}

// @param x 0=cold..1=hot
float3 Colorize(float x)
{
	x = saturate(x);

	float3 Heat = float3(1.0f, 0.0f, 0.0f);
	float3 Middle = float3(0.0f, 1.0f, 0.0f);
	float3 Cold = float3(0.0f, 0.0f, 1.0f);

	float3 ColdHeat = lerp(Cold, Heat, x);

	return lerp(Middle, ColdHeat, abs(0.5f - x) * 2);
}


// for printf debugging in the shader, does not have to be positive
// outputs a float number in the form: xx.yy
// @param LeftTop - in pixels
void PrintSmallFloatWithSign(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	int2 Cursor = LeftTop;

	bool bHasSign = Number < 0;

	// Minus Sign
	Number = abs(Number) + 0.005; // Round up first digit

	bool bIsGreater10 = Number >= 10;

	FLATTEN
	if (bHasSign && bIsGreater10)
	{
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _MINUS_);
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
		Number = abs(Number);
	}
	else if (bIsGreater10)
	{
		Cursor.x += 8;
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	}
	else if (bHasSign)
	{
		Cursor.x += 8;
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _MINUS_);
	}
	else
	{
		Cursor.x += 2*8;
	}

	// we always print this character, so no ifs needed
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
	// period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _DOT_);
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.1));
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.01));
}

// for printf debugging in the shader, does not have to be positive
// outputs a float number in the form: xx.yyy
// @param LeftTop - in pixels
void PrintMediumFloatWithSign(int2 PixelPos, inout float3 OutColor, float3 FontColor, int2 LeftTop, float Number)
{
	int2 Cursor = LeftTop;

	bool bHasSign = Number < 0;

	// Minus Sign
	Number = abs(Number) + 0.005; // Round up first digit

	bool bIsGreater10 = Number >= 10;

	FLATTEN
	if (bHasSign && bIsGreater10)
	{
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _MINUS_);
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
		Number = abs(Number);
	}
	else if (bIsGreater10)
	{
		Cursor.x += 8;
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 10));
	}
	else if (bHasSign)
	{
		Cursor.x += 8;
		PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _MINUS_);
	}
	else
	{
		Cursor.x += 2*8;
	}

	// we always print this character, so no ifs needed
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 1));
	// period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, _DOT_);
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.1));
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.01));
	// after period
	PrintCharacter(PixelPos, OutColor, FontColor, Cursor, ExtractDigitFromFloat(Number, 0.001));
}

//
void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, FStereoPSInput StereoInput, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	int2 PixelPos = (int2)SvPosition.xy;
	float2 ViewLocalUV = float2(UVAndScreenPos.z * 0.5f + 0.5f, 0.5f - 0.5f * UVAndScreenPos.w);

	// retrieve the exposure scale and target exposure scale from the eye-adaptation buffer.
	float EyeAdaptationResult = EyeAdaptationBuffer[0].r;
	float EyeAdaptationTarget = EyeAdaptationBuffer[0].g;

	float EyeAdaptationAverageSceneLuminance = EyeAdaptationBuffer[0].z;
	float EyeAdaptationExposureBiasOnly = EyeAdaptationBuffer[0].w;
	float EyeAdaptationResultWithoutExposureBias = EyeAdaptationResult / EyeAdaptationExposureBiasOnly;

	int2 OutputViewportCenter = (int2)(Output_ViewportMin + Output_ViewportSize / 2);

	float2 UVViewportCenter = (Input_UVViewportMin + Input_UVViewportSize / 2);

	const float3 NumberColor = float3(0.8, 0.85, 0.85);
	// Luminance meter at the center of the screen
	{
		const float PixelDx = abs(PixelPos.x - OutputViewportCenter.x);
		const float PixelDy = abs(PixelPos.y - OutputViewportCenter.y);
		if (PixelDx < 180 && PixelDy < 42)
		{
			float3 HDRSceneColorAvg = 0.0f;
			float3 SceneColorAvg = 0.0f;
			float SampleCount = 0.0f;
			float MeterSize = 10.0f;
			for (float OffsetX = -MeterSize; OffsetX <= MeterSize; OffsetX++)
			{
				for (float OffsetY = -MeterSize; OffsetY <= MeterSize; OffsetY++)
				{
					float2 UV = UVViewportCenter + float2(OffsetX, OffsetY) * Input_ExtentInverse;
					HDRSceneColorAvg  += Texture2DSample(HDRSceneColorTexture, HDRSceneColorSampler, UV).rgb;
					SceneColorAvg += Texture2DSample(SceneColorTexture, SceneColorSampler, UV).rgb;
					SampleCount++;
				}
			}
			HDRSceneColorAvg  = HDRSceneColorAvg / SampleCount;
			SceneColorAvg = SceneColorAvg / SampleCount;
			HDRSceneColorAvg *= View.OneOverPreExposure;

			if (PixelDx < MeterSize && PixelDy < MeterSize)
			{
				if (PixelDx == MeterSize-1 || PixelDy == MeterSize-1)
				{
					OutColor = float4(1.0, 1.0, 1.0, 1.0); // White border
				}
				else
				{
					OutColor = float4(SceneColorAvg, 1.0); // Inner visor average color
				}
			}
			else
			{
				OutColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UV); // Influenced area default scene color
			}

			float LuminanceAvg = Luminance(HDRSceneColorAvg);

			// Luminance
			int2 TopLeft = OutputViewportCenter + int2(11, 11);
			if (LuminanceAvg < 1000.0f)
			{
				PrintFloat(PixelPos, OutColor.xyz, NumberColor, TopLeft, LuminanceAvg);
				TopLeft.x += 60;
			}
			else if (LuminanceAvg < 999999.0f)
			{
				PrintFloatNoFractionLarge(PixelPos, OutColor.xyz, NumberColor, TopLeft, LuminanceAvg);
				TopLeft.x += 60;
			}
			else
			{
				PrintFloatNoFraction(PixelPos, OutColor.xyz, NumberColor, TopLeft, LuminanceAvg, 10);
				TopLeft.x += 100;
			}
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _N_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _I_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _T_);

			// EV100
			TopLeft = OutputViewportCenter + int2(11, 22);

			const float EyeAdaptationEV100 = log2((LuminanceAvg/.18)/EyeAdaptation_LuminanceMax);
			PrintMediumFloatWithSign(PixelPos, OutColor.xyz, NumberColor, TopLeft, EyeAdaptationEV100);

			TopLeft.x += 60;
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _E_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _V_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _1_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _0_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _0_);

			return;
		}
	}

	// Illuminance meter for the hemisphere in front of the camera
	if(IlluminanceMeterEnabled > 0.0f)
	{
		OutColor = float4(0.0f, 0.0f, 0.0f, 0.0f);

		// IlluminanceMeter variables must match the one in DebugProbes.usf
		const float  IlluminanceMeterSize = 20;
		const float  IlluminanceMeterHalfSize = IlluminanceMeterSize / 2;
		const float2 IlluminanceMeterCenter = OutputViewportCenter - uint2(0, 100);
		const float2 IlluminanceMeterCenterUV = IlluminanceMeterCenter * Input_ExtentInverse;

		const float PixelDx = PixelPos.x - IlluminanceMeterCenter.x;
		const float PixelDy = PixelPos.y - IlluminanceMeterCenter.y;
		const float AbsPixelDx = abs(PixelDx);
		const float AbsPixelDy = abs(PixelDy);
		if (PixelDx > -20 && PixelDx < 300 && PixelDy < 40 && PixelDy > -20)
		{
			float3 HDRSceneColorAvg = 0.0f;
			float SampleCount = 0.0f;
			LOOP
			for (float OffsetX = (-IlluminanceMeterHalfSize + 2); OffsetX < (IlluminanceMeterHalfSize-2); OffsetX++)
			{
				for (float OffsetY = (-IlluminanceMeterHalfSize + 2); OffsetY < (IlluminanceMeterHalfSize-2); OffsetY++)
				{
					float2 UV = IlluminanceMeterCenterUV + float2(OffsetX, OffsetY) * Input_ExtentInverse;
					HDRSceneColorAvg += Texture2DSampleLevel(HDRSceneColorTexture, HDRSceneColorSampler, UV, 0).rgb;
					SampleCount++;
				}
			}
			HDRSceneColorAvg = HDRSceneColorAvg / SampleCount;
			HDRSceneColorAvg *= View.OneOverPreExposure;

			OutColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UV); // Influenced area default scene color
			const int IlluminanceMeterHalfSizeInt = int(IlluminanceMeterHalfSize);
			const int AbsPixelDxInt = int(AbsPixelDx);
			const int AbsPixelDyInt = int(AbsPixelDy);
			if (AbsPixelDxInt < IlluminanceMeterHalfSizeInt && AbsPixelDyInt < IlluminanceMeterHalfSizeInt)
			{
				if (AbsPixelDxInt == (IlluminanceMeterHalfSizeInt - 1) || AbsPixelDyInt == (IlluminanceMeterHalfSizeInt-1))
				{
					OutColor = float4(1.0, 1.0, 1.0, 1.0); // White border
				}
			}

			float3 Illuminance = HDRSceneColorAvg * PI;
			float MeanIlluminance = Luminance(Illuminance);

			int2 TopLeft = IlluminanceMeterCenter + int2(11, 11);
			// Mean illuminance
			if (MeanIlluminance < 1000.0f)
			{
				PrintFloat(PixelPos, OutColor.xyz, NumberColor, TopLeft, MeanIlluminance);
				TopLeft.x += 60;
			}
			else if (MeanIlluminance < 999999.0f)
			{
				PrintFloatNoFractionLarge(PixelPos, OutColor.xyz, NumberColor, TopLeft, MeanIlluminance);
				TopLeft.x += 60;
			}
			else
			{
				PrintFloatNoFraction(PixelPos, OutColor.xyz, NumberColor, TopLeft, MeanIlluminance, 10);
				TopLeft.x += 100;
			}
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _L_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _U_);
			PrintCharacter(PixelPos, OutColor.xyz, NumberColor, TopLeft, _X_);

			// Illuminance as RGB
			TopLeft = IlluminanceMeterCenter + int2(11, 22);
			if (Illuminance.r < 1000.0f)
			{
				PrintFloat(PixelPos, OutColor.xyz, float3(1, 0.5, 0.5), TopLeft, Illuminance.r);
			}
			else if (Illuminance.r < 999999.0f)
			{
				PrintFloatNoFractionLarge(PixelPos, OutColor.xyz, float3(1, 0.5, 0.5), TopLeft, Illuminance.r);
			}
			else
			{
				PrintFloatNoFraction(PixelPos, OutColor.xyz, NumberColor, TopLeft, Illuminance.r, 10);
			}
			TopLeft.x += 100;
			if (Illuminance.g < 1000.0f)
			{
				PrintFloat(PixelPos, OutColor.xyz, float3(0.5, 1, 0.5), TopLeft, Illuminance.g);
			}
			else if (Illuminance.g < 999999.0f)
			{
				PrintFloatNoFractionLarge(PixelPos, OutColor.xyz, float3(0.5, 1, 0.5), TopLeft, Illuminance.g);
			}
			else
			{
				PrintFloatNoFraction(PixelPos, OutColor.xyz, NumberColor, TopLeft, Illuminance.g, 10);
			}
			TopLeft.x += 100;
			if (Illuminance.b < 1000.0f)
			{
				PrintFloat(PixelPos, OutColor.xyz, float3(0.5, 0.5, 1), TopLeft, Illuminance.b);
			}
			else if (Illuminance.b < 999999.0f)
			{
				PrintFloatNoFractionLarge(PixelPos, OutColor.xyz, float3(0.5, 0.5, 1), TopLeft, Illuminance.b);
			}
			else
			{
				PrintFloatNoFraction(PixelPos, OutColor.xyz, NumberColor, TopLeft, Illuminance.b, 10);
			}
			return;
		}
	}

	if (EyeAdaptation_VisualizeDebugType == 0)
	{
		OutColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UV);
	}
	else if (EyeAdaptation_VisualizeDebugType == 1)
	{
		float4 SceneColor = Texture2DSample(HDRSceneColorTexture, HDRSceneColorSampler, UV);
		SceneColor.xyz *= View.OneOverPreExposure;

		float Percentile = ComputePlaceInHistogram(SceneColor.xyz);

		OutColor.rgb = Percentile;
		if (Percentile < EyeAdaptation_ExposureLowPercent)
		{
			OutColor.rgb = float3(.5,0,0);
		}
		else if (Percentile >= EyeAdaptation_ExposureHighPercent)
		{
			OutColor.rgb = float3(0,0,.5);
		}
	}
	else if (EyeAdaptation_VisualizeDebugType == 2)
	{
		float Luminance;

		if (UsingIlluminance > 0.5f)
		{
			float2 LuminanceUV = ApplyScreenTransform(UV, ColorUVToLuminanceUV);

			Luminance = Texture2DSample(LuminanceTexture, LuminanceSampler, LuminanceUV).r;
			Luminance *= View.OneOverPreExposure;
		}
		else
		{
			float4 SceneColor = Texture2DSample(HDRSceneColorTexture, HDRSceneColorSampler, UV);
			SceneColor.xyz *= View.OneOverPreExposure;

			Luminance = CalculateEyeAdaptationLuminance(SceneColor.xyz);
		}

		OutColor.rgb = Luminance * LuminanceScale;
	}
	else // should never happen?
	{
		OutColor.rgb = float3(0,0,0);
	}

	// Compute and apply weight value at this location 
	float2 NdPos = Output_ViewportSizeInverse * (PixelPos.xy - Output_ViewportMin);
	float weight = AdaptationWeightTexture(NdPos);

	float2 IDAreaLocalUV = ViewLocalUV * 2 + float2(-1, 0);

	// left top of the border
	const int2 HistogramLeftTop = int2(Output_ViewportMin.x + 64, Output_ViewportMax.y - 128 - 32);
	const int2 HistogramSize = int2(Output_ViewportMax.x - Output_ViewportMin.x - 64 * 2, 128);
	const int HistogramOuterBorder = 4;

	// (0, 0) .. (1, 1)
	float2 InsetPx = PixelPos - HistogramLeftTop;
	float2 InsetUV = InsetPx / HistogramSize;
	
	const float3 BorderColor = Colorize(InsetUV.x);

	float BorderDistance = ComputeDistanceToRect(PixelPos, HistogramLeftTop, HistogramSize);
	
	// thin black border around the histogram
	OutColor.xyz = lerp(float3(0, 0, 0), OutColor.xyz, saturate(BorderDistance - (HistogramOuterBorder + 2)));

	// big solid border around the histogram
	OutColor.xyz = lerp(BorderColor, OutColor.xyz, saturate(BorderDistance - (HistogramOuterBorder + 1)));

	// thin black border around the histogram
	OutColor.xyz = lerp(float3(0, 0, 0), OutColor.xyz, saturate(BorderDistance - 1));

	if(BorderDistance > 0)
	{
		// outside of the histogram
		return;
	}

	// inside the histogram
	uint Bucket = (uint)(InsetUV.x * HISTOGRAM_SIZE);
	float HistogramSum = ComputeHistogramSumForVis();

	float MinExposure = EyeAdaptation_MinAverageLuminance / 0.18;
	float MaxExposure = EyeAdaptation_MaxAverageLuminance / 0.18;

	if(InsetUV.x < ComputeHistogramPositionFromLuminance(MinExposure))
	{
		// < min: grey
		OutColor.xyz = lerp(OutColor.xyz, float3(0.5f, 0.5f, 0.5f), 0.5f);
	}
	else if(InsetUV.x < ComputeHistogramPositionFromLuminance(MaxExposure))
	{
		// >= min && < max: green
		OutColor.xyz = lerp(OutColor.xyz, float3(0.5f, 0.8f, 0.5f), 0.5f);
	}
	else
	{
		// >= max: grey
		OutColor.xyz = lerp(OutColor.xyz, float3(0.5f, 0.5f, 0.5f), 0.5f);
	}

	float LocalHistogramValue = GetHistogramBucketForVis(Bucket) / ComputeHistogramMax();
	if(LocalHistogramValue >= 1 - InsetUV.y)
	{
		// histogram bars
		OutColor.xyz = lerp(OutColor.xyz, Colorize(InsetUV.x), 0.5f);
	}

	{
		// HDR luminance >0
		float LuminanceVal = ComputeLuminanceFromHistogramPosition(InsetUV.x);
		// HDR > 0
		float3 AdpatedLuminance = EyeAdaptationResult * float3(LuminanceVal, LuminanceVal, LuminanceVal);
		// 0..1
		float3 TonemappedLuminance = FilmToneMap(AdpatedLuminance);
		float3 DistMask = saturate(1.0 - 100.0 * abs(TonemappedLuminance - (1.0 - InsetUV.y)));
		OutColor = lerp(OutColor, float4(1, 1, 1, 0), float4(DistMask, 0.0));
	}

	{
		float ValuePx = ComputeHistogramPositionFromLuminance(EyeAdaptationAverageSceneLuminance/.18f)  * HistogramSize.x ;
		if(abs(InsetPx.x - ValuePx) < 3)
		{
			// blue line to show the clamped percentil
			OutColor = lerp(OutColor, float4(0, 0, 1, 0), 0.5f);
		}
	}

	// eye adaptation without bias
	{
		const float OneOverEyeAdaptationResult = 1.0f/EyeAdaptationResultWithoutExposureBias;

		const float EyeAdaptationValue = ComputeHistogramPositionFromLuminance(OneOverEyeAdaptationResult);
		const float ValuePx = EyeAdaptationValue * HistogramSize.x;

		const float EyeAdaptationEV100 = log2(OneOverEyeAdaptationResult);

		PrintSmallFloatWithSign(PixelPos, OutColor.xyz, float3(1.0, 0.0, 1.0), HistogramLeftTop + int2(ValuePx + - 6 * 8 - 3, 1), EyeAdaptationEV100);

		if(abs(InsetPx.x - ValuePx) < 2 && PixelPos.y > HistogramLeftTop.y + 9)
		{
			// purple line to show the smoothed exposure
			OutColor = lerp(OutColor, float4(0.5, 0, .5, 0), 1.0f);
		}
	}

	// eye adaptation
	{
		const float OneOverEyeAdaptationResult = 1.0f / EyeAdaptationResult;

		const float EyeAdaptationValue = ComputeHistogramPositionFromLuminance(OneOverEyeAdaptationResult);
		const float ValuePx = EyeAdaptationValue * HistogramSize.x;

		const float EyeAdaptationEV100 = log2(OneOverEyeAdaptationResult/EyeAdaptation_LuminanceMax);

		PrintSmallFloatWithSign(PixelPos, OutColor.xyz, float3(1, 1, 1), HistogramLeftTop + int2(ValuePx + - 6 * 8 - 3, 1), EyeAdaptationEV100);

		if(abs(InsetPx.x - ValuePx) < 2 && PixelPos.y > HistogramLeftTop.y + 18)
		{
			// white line to show the smoothed exposure
			OutColor = lerp(OutColor, float4(1, 1, 1, 0), 1.0f);
		}
	}

	return;

}