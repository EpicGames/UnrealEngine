// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, MSAA_SAMPLE_COUNT> EditorPrimitivesDepth;
Texture2DMS<uint2, MSAA_SAMPLE_COUNT> EditorPrimitivesStencil;
#else
// Note: opengl compiler doesn't like Texture2D<float>
Texture2D EditorPrimitivesDepth;
Texture2D<uint2> EditorPrimitivesStencil;
#endif

Texture2D ColorTexture;
SamplerState ColorSampler;

Texture2D DepthTexture;
SamplerState DepthSampler;

SCREEN_PASS_TEXTURE_VIEWPORT(Color)
SCREEN_PASS_TEXTURE_VIEWPORT(Depth)
FScreenTransform ColorToDepth;

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ 
float ReconstructDeviceZ(float3 DeviceZPlane, float2 PixelOffset)
{
	return dot(DeviceZPlane, float3(PixelOffset.xy, 1));
}

// @param DeviceZPlane plane in screenspace .x: ddx(DeviceZ), y: ddy(DeviceZ) z:DeviceZ
bool IsOccluded(int2 PixelPos, int SampleID, int OffsetX, int OffsetY, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	PixelPos += int2(OffsetX, OffsetY);

	// more stable on the silhouette
	float2 ReconstructionOffset = (View.TemporalAAParams.zw * Color_Extent * Depth_ExtentInverse);

#if MSAA_SAMPLE_COUNT > 1
	float DeviceZ = EditorPrimitivesDepth.Load(PixelPos, SampleID).r;

#if !COMPILER_GLSL && !COMPILER_PSSL && !COMPILER_METAL && !COMPILER_HLSLCC
	// not yet supported on OpenGL, slightly better quality
	ReconstructionOffset += EditorPrimitivesDepth.GetSamplePosition(SampleID);
#endif

#else
	float DeviceZ = EditorPrimitivesDepth.Load(int3(PixelPos, 0)).r;
#endif

	float SceneDeviceZ = ReconstructDeviceZ(DeviceZPlane, ReconstructionOffset);

	// clamp SceneDeviceZ in (DeviceZMinMax.x .. DeviceZMinMax.z)
	// this avoids flicking artifacts on the silhouette by limiting the depth reconstruction error
	SceneDeviceZ = max(SceneDeviceZ, DeviceZMinMax.x);
	SceneDeviceZ = min(SceneDeviceZ, DeviceZMinMax.y);

	// Soft Bias with DeviceZ for best quality (larger bias than usual because SceneDeviceZ is only approximated)
	const float DeviceDepthFade = 0.000005f;
	const float fOccluded = ((SceneDeviceZ - DeviceZ) / DeviceDepthFade);
	const bool bOccluded = fOccluded <= 1.0f;

	return bOccluded;
}

// Should this sample be greyed out
bool PerSample(int2 PixelPos, int SampleID, bool IsCenterPixelOccluded, float3 DeviceZPlane, float2 DeviceZMinMax)
{
	if (IsCenterPixelOccluded)
	{
		return true;
	}

	// [0..3]: borders
	bool IsPixelOccluded[4];

	// Diagonal cross is thicker than vertical/horizontal cross.
	IsPixelOccluded[0] = IsOccluded(PixelPos, SampleID, 1, 1, DeviceZPlane, DeviceZMinMax);
	IsPixelOccluded[1] = IsOccluded(PixelPos, SampleID, -1, -1, DeviceZPlane, DeviceZMinMax);
	IsPixelOccluded[2] = IsOccluded(PixelPos, SampleID, 1, -1, DeviceZPlane, DeviceZMinMax);
	IsPixelOccluded[3] = IsOccluded(PixelPos, SampleID, -1, 1, DeviceZPlane, DeviceZMinMax);

	if (IsPixelOccluded[0] ||
		IsPixelOccluded[1] ||
		IsPixelOccluded[2] ||
		IsPixelOccluded[3])
	{
		return true;
	}

	return false;
}

float4 GetPixelGreyColor(float4 SceneColor, float SceneColorIntensity)
{
	float Grey = Luminance(SceneColor.rgb);
	float4 GreyColor = float4(Grey, Grey, Grey, 1.0);
	GreyColor = lerp(GreyColor, float4(1., 1., 1., 1.), 0.5);
	return lerp(GreyColor, SceneColor, SceneColorIntensity);
}

// Computes min and max at once.
void MinMax(inout int2 Var, int Value)
{
	Var.x = min(Var.x, Value);
	Var.y = max(Var.y, Value);
}

void MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const float2 ColorUV = UVAndScreenPos.xy;
	const int2 ColorPixelPos = int2(ColorUV * Color_Extent);

#if MSAA_SAMPLE_COUNT > 1
	const int StencilVal = EditorPrimitivesStencil.Load(ColorPixelPos, 0) STENCIL_COMPONENT_SWIZZLE;
#else
	const int StencilVal = EditorPrimitivesStencil.Load(int3(ColorPixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
#endif

	const float4 SceneColor = ColorTexture.Load(int3(ColorPixelPos, 0));

	if (StencilVal == 2)
	{
		// This is a Nanite pixel and occlusion has been handled already
		OutColor = SceneColor;
	}
	else if (StencilVal != 0)
	{
		const float2 DepthUV = ApplyScreenTransform(ColorUV, ColorToDepth);
		const float2 DepthUVColorPixelSize = Pow2(Color_ExtentInverse) * Depth_Extent;
		const float2 DepthUVOffsets[4] = {
			{ -DepthUVColorPixelSize.x,  0 },
			{ DepthUVColorPixelSize.x,  0 },
			{ 0, -DepthUVColorPixelSize.y },
			{ 0, DepthUVColorPixelSize.y },
		};

		float Center = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV, 0).r;
		float Left = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + DepthUVOffsets[0], 0).r;
		float Right = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + DepthUVOffsets[1], 0).r;
		float Top = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + DepthUVOffsets[2], 0).r;
		float Bottom = Texture2DSampleLevel(DepthTexture, DepthSampler, DepthUV + DepthUVOffsets[3], 0).r;

		// This allows to reconstruct depth with a small pixel offset without many texture lookups (4xMSAA * 5 neighbors -> 20 samples)
		// It's an approximation assuming the surface is a plane.
		float3 DeviceZPlane;
		DeviceZPlane.x = (Right - Left) / 2;
		DeviceZPlane.y = (Bottom - Top) / 2;
		DeviceZPlane.z = Center;

		float2 DeviceZMinMax;
		DeviceZMinMax.x = min(Center, min(min(Left, Right), min(Top, Bottom)));
		DeviceZMinMax.y = max(Center, max(max(Left, Right), max(Top, Bottom)));

		bool IsPixelOccluded = IsOccluded(ColorPixelPos, 0, 0, 0, DeviceZPlane, DeviceZMinMax);

		int Sum = 0;
#if COMPILER_GLSL && FEATURE_LEVEL >= FEATURE_LEVEL_SM4
		UNROLL
#endif
		for (int SampleID = 0; SampleID < MSAA_SAMPLE_COUNT; ++SampleID)
		{
			if (PerSample(ColorPixelPos, SampleID, IsPixelOccluded, DeviceZPlane, DeviceZMinMax))
			{
				++Sum;
			}
		}

		const float SceneColorIntensity = Sum / MSAA_SAMPLE_COUNT;
		OutColor = GetPixelGreyColor(SceneColor, SceneColorIntensity);
	}
	else
	{
		// Always grey out stencil val == 0
		const float SceneColorIntensity = 0.0;
		OutColor = GetPixelGreyColor(SceneColor, SceneColorIntensity);
	}
}
