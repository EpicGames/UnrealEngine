// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ScreenPass.ush"
#include "PostProcessCommon.ush"
#include "PostProcessHistogramCommon.ush"
#include "DeferredShadingCommon.ush"
#include "TonemapCommon.ush"

StructuredBuffer<float4> EyeAdaptationBuffer;

Texture2D HDRSceneColorTexture;
Texture2D SceneColorTexture;

#if !EXPOSURE_FUSION
Texture3D LumBilateralGrid;
Texture2D BlurredLogLum;
#else
FScreenTransform ColorToExposureFusion;
SCREEN_PASS_TEXTURE_VIEWPORT(ExposureFusion)
Texture2D ExposureFusionTexture;
Texture2D ExposuresTexture;
Texture2D WeightsTexture;
#endif

SamplerState BilinearClampSampler;

uint DebugMode;

#define VISUALIZE_LOCAL_EXPOSURE_MODE_DEFAULT			(1)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_THRESHOLDS		(2)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_BASE_LUMINANCE	(3)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_DETAIL_LUMINANCE	(4)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_VALID_LOOKUP		(5)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_BASE		(6)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_SHADOWS	(7)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_HIGHLIGHTS	(8)
#define VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_WEIGHTS	(9)

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

float DiagonalStripesPattern(float2 ScreenPos, float Thickness)
{    
    float V = frac((ScreenPos.x + ScreenPos.y) / Thickness);
    return step(0.5f, V);
}

float4 MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, FStereoVSOutput Stereo, float4 SvPosition : SV_POSITION) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
	float2 ViewportUV = (int2(SvPosition.xy) - Output_ViewportMin) * Output_ViewportSizeInverse;

	float2 ExposureScaleMiddleGrey = EyeAdaptationBuffer[0].xw;

	float4 HDRSceneColor = Texture2DSample(HDRSceneColorTexture, BilinearClampSampler, UV) * View.OneOverPreExposure;
	float4 SceneColor = Texture2DSample(SceneColorTexture, BilinearClampSampler, UV);

	float LuminanceVal = CalculateEyeAdaptationLuminance(HDRSceneColor.rgb);
	float LogLuminance = log2(LuminanceVal);
	float MiddleGreyLumValue = log2(0.18 * ExposureScaleMiddleGrey.y * LocalExposure_MiddleGreyExposureCompensation);

#if !EXPOSURE_FUSION
	float BaseLogLum = CalculateBaseLogLuminance(LogLuminance, LocalExposure_BlurredLuminanceBlend, ExposureScaleMiddleGrey.x, ViewportUV, LumBilateralGrid, BlurredLogLum, BilinearClampSampler, BilinearClampSampler);
	float LocalExposure = CalculateLocalExposure(LogLuminance + log2(ExposureScaleMiddleGrey.x), BaseLogLum, MiddleGreyLumValue, LocalExposure_HighlightContrastScale, LocalExposure_ShadowContrastScale, LocalExposure_DetailStrength);
	float DetailLogLum = (LogLuminance + log2(ExposureScaleMiddleGrey.x)) - BaseLogLum;
#else
	float BaseLogLum = LogLuminance + log2(ExposureScaleMiddleGrey.x);

	float2 ExposureFusionUV = ApplyScreenTransform(UV.xy, ColorToExposureFusion);
	ExposureFusionUV = clamp(ExposureFusionUV, ExposureFusion_UVViewportBilinearMin, ExposureFusion_UVViewportBilinearMax);

	float4 Exposures = Texture2DSample(ExposuresTexture, BilinearClampSampler, ExposureFusionUV);
	float4 Weights = Texture2DSample(WeightsTexture, BilinearClampSampler, ExposureFusionUV);

	const float LuminanceVal2 = CalculateEyeAdaptationLuminance(HDRSceneColor.rgb * ExposureScaleMiddleGrey.x);
	float FusionVal = Texture2DSample(ExposureFusionTexture, BilinearClampSampler, ExposureFusionUV).x;
	float LocalExposure = CalculateFusionLocalExposure(LuminanceVal2, FusionVal);
#endif

	float BaseCentered = (BaseLogLum - MiddleGreyLumValue);

	float3 OutValue = 0;

	if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_DEFAULT)
	{
		OutValue = saturate(abs(log2(LocalExposure) / 2)) * lerp(float3(1.0f, 0.0f, 0.0f), float3(0.0f, 1.0f, 0.0f), step(LocalExposure, 1));
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_THRESHOLDS)
	{
		OutValue = SceneColor.rgb;

		if (BaseCentered > 0)
		{
			if (BaseCentered - LocalExposure_HighlightThreshold > 0)
			{
				OutValue = lerp(OutValue, float3(0,1,0), DiagonalStripesPattern(SvPosition.xy, 32.0f));
			}
		}
		else
		{
			if (BaseCentered + LocalExposure_ShadowThreshold < 0)
			{
				OutValue = lerp(OutValue, float3(1,0,0), DiagonalStripesPattern(SvPosition.xy, 32.0f));
			}
		}
	}
#if !EXPOSURE_FUSION
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_BASE_LUMINANCE)
	{
		float ContrastScale = BaseCentered > 0 ? LocalExposure_HighlightContrastScale : LocalExposure_ShadowContrastScale;

		// Apply threshold
		float ThresholdOffset;
		{
			if (BaseCentered > 0)
			{
				ThresholdOffset = BaseCentered - max(0.0f, BaseCentered - LocalExposure_HighlightThreshold);
			}
			else
			{
				ThresholdOffset = BaseCentered - min(0.0f, BaseCentered + LocalExposure_ShadowThreshold);
			}

			BaseCentered -= ThresholdOffset;
		}

		OutValue = exp2(MiddleGreyLumValue + ThresholdOffset + BaseCentered * ContrastScale);
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_DETAIL_LUMINANCE)
	{
		OutValue = exp2(DetailLogLum * LocalExposure_DetailStrength);
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_VALID_LOOKUP)
	{
		OutValue = SceneColor.rgb;

		float3 BilateralGridUVW;
		BilateralGridUVW.xy = ViewportUV * LocalExposure_BilateralGridUVScale;
		BilateralGridUVW.z = (ComputeHistogramPositionFromLogLuminance(LogLuminance) * (BILATERAL_GRID_DEPTH - 1) + 0.5f) / BILATERAL_GRID_DEPTH;

		float2 BilateralGridLum = Texture3DSample(LumBilateralGrid, BilinearClampSampler, BilateralGridUVW).xy;

		if (BilateralGridLum.y < 0.001)
		{
			// bilateral grid doesn't have valid data
			OutValue = float3(1,0,0);
		}
		else
		{
			OutValue = lerp(OutValue, float3(0,1,0), 0.5f);
		}
	}
#else
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_BASE)
	{
		OutValue = Exposures.r;
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_HIGHLIGHTS)
	{
		OutValue = Exposures.g;
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_SHADOWS)
	{
		OutValue = Exposures.b;
	}
	else if (DebugMode == VISUALIZE_LOCAL_EXPOSURE_MODE_FUSION_WEIGHTS)
	{
		// swap rb channels so colors match other visualizations
		// red = shadows
		// green = highlights
		float T = Weights.r;
		Weights.r = Weights.b;
		Weights.b = T;
		OutValue = Weights.rgb;
	}
#endif

	return float4(OutValue, 1.0f);
}