// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"


//------------------------------------------------------- PARAMETERS

uint2 OriginalFilmGrainTextureSize;
Texture2D ReducedFilmGrainTexture;
RWStructuredBuffer<float4> FilmGrainConstantsOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(1, 1, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 ReducedFilmGrainEnergy = ReducedFilmGrainTexture[uint2(0, 0)];

	float FilmGrainPixelCount = float(OriginalFilmGrainTextureSize.x * OriginalFilmGrainTextureSize.y);

	float4 FilmGrainDecodeMultiply = FilmGrainPixelCount.xxxx / ReducedFilmGrainEnergy;

	FilmGrainConstantsOutput[0] = FilmGrainDecodeMultiply;
}
