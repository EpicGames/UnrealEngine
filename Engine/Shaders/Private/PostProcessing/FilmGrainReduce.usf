// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"


//------------------------------------------------------- CONSTANTS

#define TILE_SIZE 8


//------------------------------------------------------- PARAMETERS

uint2 FilmGrainTextureSize;
Texture2D FilmGrainTexture;

RWTexture2D<float4> Output;


//------------------------------------------------------- LDS

groupshared float4 SharedColor[TILE_SIZE * TILE_SIZE];


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 InputPixelPos = TILE_SIZE * GroupId + uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);

	// Init LDS
	float4 TexelColor = FilmGrainTexture[InputPixelPos];
	{
		bool bValidPixel = all(InputPixelPos < FilmGrainTextureSize);

		if (!bValidPixel)
		{
			TexelColor = 0.0;
		}

		SharedColor[GroupThreadIndex] = TexelColor;
	}

	GroupMemoryBarrierWithGroupSync();

	// Reduction
	{
		UNROLL
		for (uint i = 32; i > 1; i /= 2)
		{
			TexelColor += SharedColor[GroupThreadIndex ^ i];

			GroupMemoryBarrierWithGroupSync();

			SharedColor[GroupThreadIndex] = TexelColor;
			
			GroupMemoryBarrierWithGroupSync();
		}
		
		TexelColor += SharedColor[GroupThreadIndex ^ 0x1];
	}

	// Output
	if (GroupThreadIndex == 0)
	{
		Output[GroupId] = TexelColor;
	}
}
