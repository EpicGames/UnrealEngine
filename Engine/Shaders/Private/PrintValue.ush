// Copyright Epic Games, Inc. All Rights Reserved.

// Simple numeric print functions. 
//   Text size and placement is entirely determined by the scale and origin of the coordinates passed in
//   Numbers appear vertically y=0 and y=1 with the baseline at y=0
//   Numbers appear horizontally right-justified at x=0 (so only at *negative* values of x)
//
// Basic usage: 
//    OutColor = lerp(OutColor, TextColor, PrintFixed(coord, 3, value));
//    OutColor = lerp(OutColor, TextColor, PrintScientific(coord, 3, value));
//
// Typical usage for a single number centered in the middle of the screen
//    OutColor = lerp(OutColor, TextColor, PrintFixed(50 * (SVPosition.xy * View.BufferSizeAndInvSize.zw - 0.5), 2, value)
//
// Typical usage for a grid of numbers, this one is every two lines in Y and every 10 characters in X
//    OutColor = lerp(OutColor, PrintFixed(float2(
//			10 * frac(6 * SVPos.x * View.BufferSizeAndInvSize.z) - 9,
//			2 * frac(30 * SVPos.y * View.BufferSizeAndInvSize.w)),
//			2, value));
//
// position can also be texture coordinates, shadow coordinates, etc.


// fixed-point print adapted from P_Malin https://www.shadertoy.com/view/4sBSWW
// converted to HLSL, font glyphs flipped vertically, use integer types, less compact style
//   @param StringCoords  position and scale of text. Numbers are 1x1 units in size, right-justified at 0,0
//   @param Value         float value to print
//   @return  0 for non-text, 1 for text values.
float PrintFixed(float2 StringCoords, int DecimalPlaces, float Value)
{
    if ((StringCoords.y < 0.0) || (StringCoords.y >= 1.0)) return 0.0;
	int iCoord = -int(floor(StringCoords.x));

    int NegativeValue = Value < 0.0 ? 1 : 0;
    Value = abs(Value);
	int Digits = max(int(log2(Value) / log2(10.0)), 0) + 1;
	int Decimal = DecimalPlaces > 0 ? 1 : 0;
	int iValue = int(floor(Value * pow(10., DecimalPlaces)));

	int CharBin = 0, DigitIndex = 0;
	if (iCoord < 0)
		return 0.;									// right of number
	else if (iCoord < DecimalPlaces)
		DigitIndex = iCoord;						// fractional digit
	else if (iCoord == DecimalPlaces && DecimalPlaces)
		CharBin = 131072;							// decimal point
	else if (iCoord < Digits + Decimal + DecimalPlaces)
		DigitIndex = iCoord - Decimal;				// integer digit
	else if (iCoord < NegativeValue + Digits + Decimal + DecimalPlaces)
		CharBin = 1792;								// minus sign
	else
		return 0.;									// left of number

	// has to be a digit
	if (CharBin == 0)
    {
		const int digits[] = { 480599, 139810, 464711, 476999, 280405, 476951, 481047, 279623, 481111, 477015 };
        int Digit = int(iValue / pow(10, DigitIndex)) % 10;
        CharBin = digits[Digit];
    }

	// convert binary to 4x5 array of points
    int2 DigitCoord = int2(frac(StringCoords.x) * 4., StringCoords.y * 5.);
    return float((CharBin >> (DigitCoord.x + DigitCoord.y * 4)) & 1);
}

// Print value in scientific notation
//   @param StringCoords  position and scale of text. Numbers are 1x1 units in size, right-justified at 0,0
//   @param DecimalPlaces number of decimal places to print in mantissa
//   @param Value         float value to print
//   @return  0 for non-text, 1 for text values.
// Typical usage: OutColor = lerp(OutColor, TextColor, PrintValue(coord, 3, value));
float PrintScientific(float2 StringCoords, int DecimalPlaces, float Value)
{
	if ((StringCoords.y < 0.0) || (StringCoords.y >= 1.0)) return 0.0;
	int iCoord = -int(floor(StringCoords.x));

	int NegativeValue = Value < 0.0 ? 1 : 0;
	Value = abs(Value);

	int Exponent = int(floor(log2(Value) / log2(10.)));
	int NegativeExponent = Exponent < 0 ? 1 : 0;
	Exponent = abs(Exponent);
	int ExponentDigits = max(0, int(floor(log2(float(Exponent)) / log2(10.)))) + 1;

	int iValue = int(floor(Value * pow(10., DecimalPlaces - Exponent)));

	int CharBin = 0, DigitIndex = 0;
	if (iCoord < 0)
		return 0.;									// right of number
	else if (iCoord < ExponentDigits)
	{
		iValue = Exponent;							// set up to print exponent
		DigitIndex = iCoord;
	}
	else if (iCoord == ExponentDigits)
		CharBin = (NegativeExponent == 0) ? 10016 : 1792;	// +/- sign for exponent
	else if (iCoord == ExponentDigits + 1)
		CharBin = 464663;									// E for exponent
	else if (iCoord < ExponentDigits + DecimalPlaces + 2)
		DigitIndex = iCoord - ExponentDigits - 1;			// fractional digit
	else if (iCoord == ExponentDigits + DecimalPlaces + 2)
		CharBin = 131072;									// decimal point
	else if (iCoord == ExponentDigits + DecimalPlaces + 3)
		DigitIndex = DecimalPlaces;							// integer digit
	else if (iCoord == ExponentDigits + DecimalPlaces + 3 + NegativeValue)
		CharBin = 1792;								// minus sign for value
	else
		return 0.;									// left of number

	if (CharBin == 0)												// has to be a digit
	{
		const int digits[] = { 480599, 139810, 464711, 476999, 280405, 476951, 481047, 279623, 481111, 477015 };
		int Digit = iValue * pow(10, -DigitIndex) % 10;
		CharBin = digits[Digit];
	}

	// convert binary to 4x5 array of points
	int2 DigitCoord = int2(frac(StringCoords.x) * 4., StringCoords.y * 5.);
	return float((CharBin >> (DigitCoord.x + DigitCoord.y * 4)) & 1);
}
