// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	QuadComplexityAccumulatePixelShader.usf: Outputs complexity
=============================================================================*/

#include "Common.ush"

float4 NormalizedComplexity;
bool bShowQuadOverdraw;

#include "QuadOverdraw.ush"
#include "DebugViewModeCommon.ush"

#if OUTPUT_QUAD_OVERDRAW

// Needed because UAV accesses disable early depth test otherwises.
EARLYDEPTHSTENCIL
void Main(
	in FDebugPSInLean Input,
	out float4 OutColor : SV_Target0
	)
{
	float3 FinalComplexity = NormalizedComplexity.xyz;

	[branch]
	if (bShowQuadOverdraw && NormalizedComplexity.x > 0)
	{
		// Here we can give a high iteraction count because clip() gives an early out.
		FinalComplexity.x = ComputeQuadCoverage(Input.SvPosition.xy, Input.PrimitiveID, 128, true, true, FloatToComplexity(FinalComplexity.x));
	}

	// use the maximum range allowed for scene color
	// alpha channel needs to be 1 to have decals working where the framebuffer blend is setup to multiply with alpha
	OutColor = RETURN_COLOR(float4(FinalComplexity.xyz, 1));
}

#else

void Main(out float4 OutColor : SV_Target0)
{
	// use the maximum range allowed for scene color
	// alpha channel needs to be 1 to have decals working where the framebuffer blend is setup to multiply with alpha
	OutColor = RETURN_COLOR(float4(NormalizedComplexity.xyz, 1));
}

#endif // OUTPUT_QUAD_OVERDRAW