// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	QuadOverdraw.ush: Utils to compute quad coverage
=============================================================================*/

#pragma once

// Because we use the same UAV for Descriptors and Complexity, we have the same clear value.
// In this case, the invalid descriptor is 0, so we need to add 1 to the primitive ID to prevent clash.
const static uint QO_DESC_NONE = 0;

// Complexity is 16 bits interger, 16 bits fractionnal.
const static uint QO_FRACTIONNAL_COMPLEXITY = 0xFFFF;
 
// Those are the states for the quad overdraw processing (QOS)
const static int QOS_ABORT = -2;
const static int QOS_DONE = -1;
const static int QOS_PENDING = 0;
const static int QOS_SYNCHRONIZING = 1;
const static int QOS_OWNER = 2;

uint GetPrimitiveID(uint Desc)
{
	return (Desc >> 2) - 1;
}

uint GetQuadPixelCount(uint Desc)
{
	return Desc & 0x3;
}

uint GenerateDesc(uint SvPrimitiveID)
{
	return (SvPrimitiveID + 1) << 2;
}

uint FloatToComplexity(float Value)
{
	return (uint)(Value * (float)QO_FRACTIONNAL_COMPLEXITY); 
}

float ComplexityToFloat(uint Complexity)
{
	return (float)Complexity / (float)QO_FRACTIONNAL_COMPLEXITY;
}

#if SHADING_PATH_MOBILE
#define RWQuadBuffer MobileBasePass.QuadOverdraw
#else
#define RWQuadBuffer DebugViewModePass.QuadOverdraw
#endif

#if OUTPUT_QUAD_OVERDRAW

// The temporary buffer used to synchronize and exchange data between quad sub-pixels.
// Left half hold QuadDescriptor, right half hold QuadComplexity
// Both are halfres here.


// The QuadComplexity is the same resource as the QuadDescriptor with an offset in X.
// This is only required to prevent binding an additional UAV which would exceed the allowed RT + UAV count.
#define RWQuadDescriptor RWQuadBuffer
#define RWQuadComplexity RWQuadBuffer
// RWTexture2D<uint> RWQuadComplexity

uint2 QuadComplexityOffset()
{
	uint QuadBufferWidth, QuadBufferHeight;
	RWQuadBuffer.GetDimensions(QuadBufferWidth, QuadBufferHeight);
	return uint2(QuadBufferWidth / 2, 0);
}

/** 
	Compute the coverage for the given Pixel-Primitive.
	
	bOwnerOnly : if true, the coverage will only be computed for one of the quad pixels, and will be 0 for the others (less expensive)
	
	bOutputToComplexity : if complexity is written to the UAV instead of RT. 
		This is the only code path enabling early out in the costly loop. 
		Use with [earlydepthstencil] to keep depth write and update.
		A major behavior difference is that complexity is always additive. (can be changed)
	
	QuadComplexity : the value of the quad complexity.
*/
uint ComputeQuadCoverage(uint2 SvPosition, uint SvPrimitiveID, uniform int NumIteration, uniform bool bOwnerOnly, uniform bool bOutputToComplexity, uint QuadComplexity)
{
	uint2 QuadID = SvPosition.xy / 2;
	int State = QOS_PENDING;

	// Excluding the owner. By default we put the maximum value so that if failed to process, the scale will be one.
	// Starting with 3 also enables a quick completion if all pixels are updated. (once at 3, it can not be increased furthermore)
	uint QuadPixelCount = 3;

	// Because several primitives could be accessing the same quad, we need to loop enough iteration for everything to sync.
	[loop]
    for (int i = 0; i < NumIteration; i++)
	{
		// When outputting to complexity, we assume there are no valid rendertarget update, aside depth buffer.
		// Depth to still be updated correctly, the shader calling this needs to activate [earlydepthstencil]
		if (bOutputToComplexity)
		{
			clip(State);
		}

		[branch]
		if (!bOwnerOnly && State == QOS_SYNCHRONIZING) // bOwnerOnly don't use this path.
		{
			uint CurrDesc = RWQuadDescriptor[QuadID];

			// If the primitive ID has changed, then the owner has finished its process.
			[flatten]
			if (GetPrimitiveID(CurrDesc) != SvPrimitiveID)
			{
				State = QOS_DONE;
			}
			else
			{
				QuadPixelCount = GetQuadPixelCount(CurrDesc);
			}
		}

		[branch]
		if (State == QOS_OWNER)
		{
			uint CurrCount = GetQuadPixelCount(RWQuadDescriptor[QuadID]);

			// If the count is not increasing, stop now.
			[branch]
			if (CurrCount == QuadPixelCount)
			{
				RWQuadDescriptor[QuadID] = QO_DESC_NONE;
				State = QOS_DONE;

				if (bOutputToComplexity)
				{
					InterlockedAdd(RWQuadComplexity[QuadID + QuadComplexityOffset()], QuadComplexity);
				}
			}
			else
			{
				QuadPixelCount = CurrCount;
			}
		}

		[branch]
		if (State == QOS_PENDING)
		{
			uint PrevDesc;
			InterlockedCompareExchange(RWQuadDescriptor[QuadID], QO_DESC_NONE, GenerateDesc(SvPrimitiveID), PrevDesc);

			// If no primitive was processing this quad, then this pixel owns it.
			[flatten]
			if (PrevDesc == QO_DESC_NONE)
			{
				State = QOS_OWNER;
			}

			// If another pixel from the same primitive is the owner, start synchronizing.
			[branch]
			if (GetPrimitiveID(PrevDesc) == SvPrimitiveID)
			{
				InterlockedAdd(RWQuadDescriptor[QuadID], 1);

				State = bOwnerOnly ? QOS_ABORT : QOS_SYNCHRONIZING;
			}
		}
	}

	// This is required in case the number of iteration was too small, release the ownership of the quad.
	[branch]
	if (State == QOS_OWNER)
	{
		RWQuadDescriptor[QuadID] = QO_DESC_NONE;
	}

	if (bOutputToComplexity)
	{
		return 0;
	}
	else
	{
		return State != QOS_ABORT ? (1 + QuadPixelCount) : 0;
	}
}

#endif // OUTPUT_QUAD_OVERDRAW
