// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Quantization.ush: Utility that quantizes floats for writing to render targets
=============================================================================*/

#pragma once

#include "Random.ush"
#include "MonteCarlo.ush"
#include "RandomInterleavedGradientNoise.ush"

float3 QuantizeFloatColor(in float3 Color, in float3 QuantizationError, in float E)
{
	float3 Error = Color * QuantizationError;

	Error[0] = asfloat(asuint(Error[0]) & ~0x007FFFFF);
	Error[1] = asfloat(asuint(Error[1]) & ~0x007FFFFF);
	Error[2] = asfloat(asuint(Error[2]) & ~0x007FFFFF);
	
	return Color + Error * E;
}


#define QUANTIZE_NOISE_TYPE					uint
#define QUANTIZE_NOISE_HAMMERSLEY			0
#define QUANTIZE_NOISE_INTERLEAVEDGRADIENT	1

float3 QuantizeFloatColor(in float3 Color, in float3 QuantizationError, in float2 Pos2D, in QUANTIZE_NOISE_TYPE NoiseType)
{
	float E = 0.0;

	if (NoiseType == QUANTIZE_NOISE_HAMMERSLEY)
	{
		uint Random = Rand3DPCG16(int3(Pos2D, View.StateFrameIndexMod8)).x;
		E = Rand16ToFloat(Random);
	}
	else // QUANTIZE_NOISE_INTERLEAVEDGRADIENT
	{
		E = InterleavedGradientNoise(Pos2D, View.StateFrameIndexMod8);
	}

	return QuantizeFloatColor(Color, QuantizationError, E);
}
