// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define FQuat half4

// Unless otherwise specified, use lerp + normalize to interpolate quats
#ifndef QUAT_SLERP_APPROXIMATE
	#define QUAT_SLERP_APPROXIMATE 1
#endif

FQuat QuatIdentity()
{
	return FQuat(0, 0, 0, 1);
}

FQuat QuatFromAxisAngle(in half3 AxisN, in float AngleRad)
{
	float S, C;
	sincos(AngleRad * 0.5f, S, C);

	return FQuat(AxisN * S, C);
}

FQuat QuatConjugate(in FQuat Q)
{
	return FQuat(-Q.xyz, Q.w);
}

float3 QuatRotateVector(in FQuat Q, in float3 V)
{
	float3 T = 2.0f * cross(Q.xyz, V);
	return V + Q.w * T + cross(Q.xyz,T);
}

FQuat QuatMultiply(in FQuat Q1, in FQuat Q2)
{
	return FQuat(
		(Q1.w * Q2.x) + (Q1.x * Q2.w) + (Q1.y * Q2.z) - (Q1.z * Q2.y),
		(Q1.w * Q2.y) - (Q1.x * Q2.z) + (Q1.y * Q2.w) + (Q1.z * Q2.x),
		(Q1.w * Q2.z) + (Q1.x * Q2.y) - (Q1.y * Q2.x) + (Q1.z * Q2.w),
		(Q1.w * Q2.w) - (Q1.x * Q2.x) - (Q1.y * Q2.y) - (Q1.z * Q2.z)
	);
}

half3x3 QuatToMatrix(in FQuat Q)
{
	return half3x3(
		QuatRotateVector(Q, float3(1, 0, 0)),
		QuatRotateVector(Q, float3(0, 1, 0)),
		QuatRotateVector(Q, float3(0, 0, 1))
	);
}

FQuat QuatFromMatrix(in half3x3 M)
{
	FQuat Q;
	half T;

	if (M[2][2] < 0.0f)
	{
		if (M[0][0] > M[1][1])
		{
			T = 1.0f + M[0][0] - M[1][1] - M[2][2];
			Q = FQuat(T, M[0][1] + M[1][0], M[2][0] + M[0][2], M[1][2] - M[2][1]);
		}
		else
		{
			T = 1.0f - M[0][0] + M[1][1] - M[2][2];
			Q = FQuat(M[0][1] + M[1][0], T, M[1][2] + M[2][1], M[2][0] - M[0][2]);
		}
	}
	else
	{
		if (M[0][0] < -M[1][1])
		{
			T = 1.0f - M[0][0] - M[1][1] + M[2][2];
			Q = FQuat(M[2][0] + M[0][2], M[1][2] + M[2][1], T, M[0][1] - M[1][0]);
		}
		else
		{
			T = 1.0f + M[0][0] + M[1][1] + M[2][2];
			Q = FQuat(M[1][2] - M[2][1], M[2][0] - M[0][2], M[0][1] - M[1][0], T);
		}
	}

	return Q * 0.5f * rsqrt(T);
}

FQuat QuatSlerpPrecise(FQuat Q0, FQuat Q1, half Slerp)
{
	// Get the half angle between quats.
	const half C = dot(Q0, Q1);
	const half AbsC = abs(C);

	half Scale0, Scale1;
	if (AbsC > 0.999f)
	{
		// Fall back to lerp when the angle between is tiny (prevents NaN from 1/sin(0))
		Scale0 = half(1.0) - Slerp;
		Scale1 = Slerp;
	}
	else
	{
		const half HalfAngle = acos(AbsC);
		const half InvS = rcp(sin(HalfAngle));
		Scale0 = sin((half(1.0) - Slerp) * HalfAngle) * InvS;
		Scale1 = sin(Slerp * HalfAngle) * InvS;
	}
	
	// Negate Scale1 if angle > 180 degrees to lerp in the shorter direction
	Scale1 = C < 0.0f ? -Scale1 : Scale1;

	return Scale0 * Q0 + Scale1 * Q1;
}

FQuat QuatSlerpApproximate(FQuat Q0, FQuat Q1, half Slerp)
{
	// Just use linear interpolation and normalize. Negate Q1 if the angle between them is
	// greater than 180 degrees to lerp in the shorter direction.
	const half C = dot(Q0, Q1); // cosine of the half angle between them
	return normalize(lerp(Q0, C < 0.0f ? -Q1 : Q1, Slerp));
}

FQuat QuatSlerp(FQuat Q0, FQuat Q1, half Slerp)
{
#if QUAT_SLERP_APPROXIMATE
	return QuatSlerpApproximate(Q0, Q1, Slerp);
#else
	return QuatSlerpPrecise(Q0, Q1, Slerp);
#endif
}

FQuat QuatNormalize(in FQuat Q)
{
	float Len2 = dot(Q, Q);
	if (Len2 > 0.0f)
	{
		return Q * rsqrt(Len2);
	}
	else
	{
		return QuatIdentity();
	}
}