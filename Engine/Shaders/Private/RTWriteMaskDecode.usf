// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"


#ifdef OVERRIDE_RTWRITEMASKPROCESSING_USH

	#include "/Platform/Private/RTWriteMaskProcessing.usf"

#else

	[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
	void RTWriteMaskCombineMain(
		uint3 GroupId : SV_GroupID,
		uint3 DispatchThreadId : SV_DispatchThreadID,
		uint3 GroupThreadId : SV_GroupThreadID,
		uint GroupIndex : SV_GroupIndex ) 
	{
	}

	[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
	void RTWriteMaskDecodeSingleMain(
		uint3 GroupId : SV_GroupID,
		uint3 DispatchThreadId : SV_DispatchThreadID,
		uint3 GroupThreadId : SV_GroupThreadID,
		uint GroupIndex : SV_GroupIndex ) 
	{
	}
#endif