// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	RadixSortShaders.usf: Compute shader implementation of radix sort.
==============================================================================*/

#include "Common.ush"

/*------------------------------------------------------------------------------
	Compile time parameters:
		RADIX_BITS - The number of bits to inspect during each pass.
		THREAD_COUNT - The number of threads to launch per workgroup.
		KEYS_PER_LOOP - The number of keys to process simultaneously in a single
			thread.
		MAX_GROUP_COUNT - The maximum number of groups that will be used for the
			upsweep and downsweep.

	Notes:
		The upsweep and downsweep kernels must never be invoked with more than
		MAX_GROUP_COUNT workgroups.

		The spine kernel does not respect THREAD_COUNT and will launch the 
		number of threads needed to efficiently process MAX_GROUP_COUNT *
		(1 << RADIX_BITS) counters. The spine kernel should always be invoked
		with a single workgroup.

		The tile size is defined as THREAD_COUNT * KEYS_PER_LOOP. When computing
		the number of tiles and groups needed to execute the sort, CPU code
		must respect the tile size with which these shaders have been compiled.
------------------------------------------------------------------------------*/

/** Full group sync as some GPUs will hang without it, see UE-282658. */
#define PPS_BARRIER GroupMemoryBarrierWithGroupSync

/** The number of digits per radix. */
#define DIGIT_COUNT (1 << RADIX_BITS)
/** Bitmask to retrieve the digit of a key. */
#define DIGIT_MASK (DIGIT_COUNT - 1)

/** The number of words per bank. This is used to compute padding to avoid bank conflicts. */
#define WORDS_PER_BANK (32)
/** The padded size of a bank. */
#define PADDED_BANK_SIZE (WORDS_PER_BANK + 1)

/*------------------------------------------------------------------------------
	Common functionality.
------------------------------------------------------------------------------*/

/** Specifies which shaders use the raking technique for computing parallel prefix sums. */
#define USES_RAKING (RADIX_SORT_UPSWEEP || RADIX_SORT_SPINE || RADIX_SORT_DOWNSWEEP)

#if (RADIX_SORT_UPSWEEP || RADIX_SORT_DOWNSWEEP)

/** The size of a single tile. */
#define TILE_SIZE (THREAD_COUNT * KEYS_PER_LOOP)

/** The number of counters required per digit. */
#define COUNTERS_PER_DIGIT (THREAD_COUNT)
/** The number of banks per digit. Counters are segregated in to banks to avoid bank conflicts. */
#define BANKS_PER_DIGIT (COUNTERS_PER_DIGIT / WORDS_PER_BANK)

/** The number of raking threads. */
#define RAKING_THREAD_COUNT (64)
/** The number of raking threads used per digit. */
#define RAKING_THREADS_PER_DIGIT (RAKING_THREAD_COUNT / DIGIT_COUNT)
/** The required padding for raking thread counters. */
#define RAKING_COUNTER_PADDING (RAKING_THREADS_PER_DIGIT >> 1)
/** The number of raking counters required per digit. */
#define RAKING_COUNTERS_PER_DIGIT (RAKING_THREADS_PER_DIGIT + RAKING_COUNTER_PADDING)
/** The number of counters processed by each raking thread. */
#define COUNTERS_PER_RAKING_THREAD (COUNTERS_PER_DIGIT / RAKING_THREADS_PER_DIGIT)
/** The number of raking threads per bank. */
#define RAKING_THREADS_PER_BANK (WORDS_PER_BANK / COUNTERS_PER_RAKING_THREAD)

#elif RADIX_SORT_SPINE // #if (RADIX_SORT_UPSWEEP || RADIX_SORT_DOWNSWEEP)

/** The number of counters per digit. Note that this shader uses twice the
	number of required counters to work around a bug. */
#define COUNTERS_PER_DIGIT (MAX_GROUP_COUNT)
/** The number of banks required per digit. */
#define BANKS_PER_DIGIT (COUNTERS_PER_DIGIT / WORDS_PER_BANK)

/** The number of raking threads. */
#define RAKING_THREAD_COUNT (BANKS_PER_DIGIT * DIGIT_COUNT)
/** The number of raking threads used per digit. */
#define RAKING_THREADS_PER_DIGIT (RAKING_THREAD_COUNT / DIGIT_COUNT)
/** The amount of padding required for raking counters. */
#define RAKING_COUNTER_PADDING (RAKING_THREADS_PER_DIGIT >> 1)
/** The number of raking counters required per digit. */
#define RAKING_COUNTERS_PER_DIGIT (RAKING_THREADS_PER_DIGIT + RAKING_COUNTER_PADDING)
/** The number of counters processed by each raking thread. */
#define COUNTERS_PER_RAKING_THREAD (COUNTERS_PER_DIGIT / RAKING_THREADS_PER_DIGIT)
/** The number of raking threads per bank. */
#define RAKING_THREADS_PER_BANK (WORDS_PER_BANK / COUNTERS_PER_RAKING_THREAD)

#endif // #elif RADIX_SORT_SPINE

#if USES_RAKING

/**
 * Computes the bank offset to rake for a given thread.
 */
uint GetBankToRakeOffset( const uint ThreadId )
{
	return (ThreadId / RAKING_THREADS_PER_BANK) * PADDED_BANK_SIZE // Which bank to rake.
		+ (ThreadId & (RAKING_THREADS_PER_BANK - 1)) // Which counters within the bank to rake.
		* COUNTERS_PER_RAKING_THREAD;
}

/**
 * Computes the digit to rake for a given thread.
 */
uint GetRakingDigit( const uint ThreadId )
{
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		return ThreadId / RAKING_THREADS_PER_DIGIT;
	}
	else
	{
		return DIGIT_COUNT;
	}
}

/**
 * Computes the index to rake for a given thread.
 */
uint GetRakingIndex( const uint ThreadId )
{
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		return GetRakingDigit( ThreadId ) * RAKING_COUNTERS_PER_DIGIT // Which digit.
			+ (ThreadId & (RAKING_THREADS_PER_DIGIT - 1)) // Thread offset within digit.
			+ (RAKING_COUNTER_PADDING); // Add in padding.
	}
	else
	{
		return RAKING_COUNTERS_PER_DIGIT * DIGIT_COUNT;
	}
}

#endif // #if USES_RAKING

/*------------------------------------------------------------------------------
	The offset clearing kernel. This kernel just zeroes out the offsets buffer.

	Note that MAX_GROUP_COUNT * DIGIT_COUNT must be a multiple of THREAD_COUNT.
------------------------------------------------------------------------------*/

#if RADIX_SORT_CLEAR_OFFSETS

#define OFFSET_COUNT (MAX_GROUP_COUNT * DIGIT_COUNT)
#define OFFSETS_PER_THREAD (OFFSET_COUNT / THREAD_COUNT)

/** Output buffer to clear. */
RWBuffer<uint> OutOffsets;

[numthreads(THREAD_COUNT,1,1)]
void RadixSort_ClearOffsets(
	uint3 GroupThreadId : SV_GroupThreadID )
{
	// Determine group and thread IDs.
	const uint ThreadId = GroupThreadId.x;

	// Clear all offsets.
	for ( uint OffsetIndex = 0; OffsetIndex < OFFSETS_PER_THREAD; ++OffsetIndex )
	{
		OutOffsets[OffsetIndex * THREAD_COUNT + ThreadId] = 0;
	}
}

#endif // #if RADIX_SORT_CLEAR_OFFSETS

/*------------------------------------------------------------------------------
	The upsweep sorting kernel. This kernel performs an upsweep scan on all
	tiles allocated to this group and computes per-digit totals. These totals
	are output to the offsets buffer.
------------------------------------------------------------------------------*/

#if RADIX_SORT_UPSWEEP

/** Input keys to be sorted. */
Buffer<uint> InKeys;
/** Output buffer for offsets. */
RWBuffer<uint> OutOffsets;

/** Local storage for the digit counters. */
groupshared uint LocalCounters[BANKS_PER_DIGIT * DIGIT_COUNT * PADDED_BANK_SIZE];
/** Local storage for raking totals. */
groupshared uint LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * DIGIT_COUNT + 1];

/**
 * The upsweep sorting kernel computing a scan upsweep on tiles per thread group.
 */
[numthreads(THREAD_COUNT,1,1)]
void RadixSort_Upsweep(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupIdXYZ : SV_GroupID )
{
	uint i;
	uint FirstTileIndex;
	uint TileCountForGroup;
	uint ExtraKeysForGroup;

	// Determine group and thread IDs.
	const uint ThreadId = GroupThreadId.x;
	const uint GroupId = GroupIdXYZ.x;

	const uint BankOffset = ThreadId / (WORDS_PER_BANK);
	const uint CounterOffset = ThreadId & (WORDS_PER_BANK - 1);

	const uint BankToRakeOffset = GetBankToRakeOffset( ThreadId );		
	const uint RakingIndex = GetRakingIndex( ThreadId );

#if 0
	if ( RadixSortUB.ExtraKeyCount == 16 && RadixSortUB.TilesPerGroup != 0 )
	{
		if ( ThreadId == 0 )
		{
			OutOffsets[DIGIT_COUNT*1] = RadixSortUB.RadixShift;
			OutOffsets[DIGIT_COUNT*2] = RadixSortUB.TilesPerGroup;
			OutOffsets[DIGIT_COUNT*3] = RadixSortUB.ExtraTileCount;
			OutOffsets[DIGIT_COUNT*4] = RadixSortUB.ExtraKeyCount;
			OutOffsets[DIGIT_COUNT*5] = RadixSortUB.GroupCount;
		}
	}
#endif
		
	// Clear local counters.
	for ( uint DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex )
	{
		const uint BankIndex = DigitIndex * BANKS_PER_DIGIT + BankOffset;
		LocalCounters[BankIndex * PADDED_BANK_SIZE + CounterOffset] = 0;
	}

	// Clear the raking counter padding.
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		[unroll]
		for ( i = 1; i <= RAKING_COUNTER_PADDING; ++i )
		{
			LocalRakingTotals[RakingIndex - i] = 0;
		}
	}

	// The number of tiles to process in this group.
	if ( GroupId < RadixSortUB.ExtraTileCount )
	{
		FirstTileIndex = GroupId * (RadixSortUB.TilesPerGroup + 1);
		TileCountForGroup = RadixSortUB.TilesPerGroup + 1;
	}
	else
	{
		FirstTileIndex = GroupId * RadixSortUB.TilesPerGroup + RadixSortUB.ExtraTileCount;
		TileCountForGroup = RadixSortUB.TilesPerGroup;
	}

	// The last group has to process any keys after the last tile.
	if ( GroupId == (RadixSortUB.GroupCount - 1) )
	{
		ExtraKeysForGroup = RadixSortUB.ExtraKeyCount;
	}
	else
	{
		ExtraKeysForGroup = 0;
	}

	// Key range for this group.
	uint GroupKeyBegin = FirstTileIndex * TILE_SIZE;
	uint GroupKeyEnd = GroupKeyBegin + TileCountForGroup * TILE_SIZE;

	// Acquire LocalCounters.
	GroupMemoryBarrierWithGroupSync();

	// Accumulate digit counters for the tiles assigned to this group.
	while ( GroupKeyBegin < GroupKeyEnd )
	{
		const uint Key = InKeys[GroupKeyBegin + ThreadId];
		const uint Digit = (Key >> RadixSortUB.RadixShift) & DIGIT_MASK;
		const uint BankIndex = Digit * BANKS_PER_DIGIT + BankOffset;
		LocalCounters[BankIndex * PADDED_BANK_SIZE + CounterOffset] += 1;
		//LocalCounters[BankOffset * PADDED_BANK_SIZE + CounterOffset] += 1;
		GroupKeyBegin += THREAD_COUNT;
	}

	// Accumulate digit counters for any additional keys assigned to this group.
	GroupKeyBegin = GroupKeyEnd;
	GroupKeyEnd += ExtraKeysForGroup;

	while ( GroupKeyBegin < GroupKeyEnd )
	{
		if ( GroupKeyBegin + ThreadId < GroupKeyEnd )
		{
			const uint Key = InKeys[GroupKeyBegin + ThreadId];
			const uint Digit = (Key >> RadixSortUB.RadixShift) & DIGIT_MASK;
			const uint BankIndex = Digit * BANKS_PER_DIGIT + BankOffset;
			LocalCounters[BankIndex * PADDED_BANK_SIZE + CounterOffset] += 1;
			//LocalCounters[BankOffset * PADDED_BANK_SIZE + CounterOffset] += 100;
		}
		GroupKeyBegin += THREAD_COUNT;
	}

	// Acquire LocalCounters.
	GroupMemoryBarrierWithGroupSync();

	// Reduce.
	uint Total = 0;
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		// Perform a serial reduction on this raking thread's counters.
		Total = LocalCounters[BankToRakeOffset];
		[unroll]
		for ( i = 1; i < COUNTERS_PER_RAKING_THREAD; ++i )
		{
			Total += LocalCounters[BankToRakeOffset + i];
		}

		// Place the total in the raking counter.
		LocalRakingTotals[RakingIndex] = Total;
	}

	for ( uint RakingOffset = 1; RakingOffset < RAKING_THREADS_PER_DIGIT; RakingOffset <<= 1 )
	{
		// Acquire LocalRakingTotals.
		PPS_BARRIER();
		Total += LocalRakingTotals[RakingIndex - RakingOffset];
		LocalRakingTotals[RakingIndex] = Total;
	}

	// Acquire LocalRakingTotals.
	GroupMemoryBarrierWithGroupSync();

	// Store digit totals for this group.
	if ( ThreadId < DIGIT_COUNT )
	{
		OutOffsets[GroupId * DIGIT_COUNT + ThreadId] = LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * (ThreadId+1) - 1];
	}
}

#endif // #if RADIX_SORT_UPSWEEP

/*------------------------------------------------------------------------------
	The spine sorting kernel. This kernel performs a parallel prefix sum on
	the offsets computed by each work group in the upsweep. The outputs will be
	used by individual work groups during the downsweep to compute the final 
	location of keys.
------------------------------------------------------------------------------*/

#if RADIX_SORT_SPINE

/** The amount of padding to add for scanning digits. */
#define DIGIT_SCAN_PADDING (DIGIT_COUNT >> 1)

/** The number of threads required for this phase. */
#if COUNTERS_PER_DIGIT > RAKING_THREAD_COUNT
#define SPINE_THREAD_COUNT COUNTERS_PER_DIGIT
#else
#define SPINE_THREAD_COUNT RAKING_THREAD_COUNT
#endif

/** Input offsets to be scanned. */
Buffer<uint> InOffsets;
/** Output buffer for scanned offsets. */
RWBuffer<uint> OutOffsets;

/** Local storage for the digit counters. */
groupshared uint LocalOffsets[BANKS_PER_DIGIT * DIGIT_COUNT * PADDED_BANK_SIZE];
/** Local storage for raking totals. */
groupshared uint LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * DIGIT_COUNT + 1];
/** Local storage for the per-digit totals. */
groupshared uint LocalTotals[128 + DIGIT_SCAN_PADDING];

/**
 * Spine parallel prefix sum kernel.
 */
[numthreads(SPINE_THREAD_COUNT,1,1)]
void RadixSort_Spine(
	uint3 GroupThreadId : SV_GroupThreadID )
{
	uint i;

	// Determine the index for this thread.
	const uint ThreadId = GroupThreadId.x;

	const uint BankOffset = ThreadId / WORDS_PER_BANK;
	const uint CounterOffset = ThreadId & (WORDS_PER_BANK - 1);

	const uint BankToRakeOffset = GetBankToRakeOffset( ThreadId );
	const uint RakingDigit = GetRakingDigit( ThreadId );
	const uint RakingIndex = GetRakingIndex( ThreadId );

	// Load offsets.
	if ( ThreadId < MAX_GROUP_COUNT )
	{
		for ( uint DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex )
		{
			const uint BankIndex = DigitIndex * BANKS_PER_DIGIT + BankOffset;
			LocalOffsets[BankIndex * PADDED_BANK_SIZE + CounterOffset] = InOffsets[ThreadId * DIGIT_COUNT + DigitIndex];
		}
	}
	else
	{
		for ( uint DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex )
		{
			const uint BankIndex = DigitIndex * BANKS_PER_DIGIT + BankOffset;
			LocalOffsets[BankIndex * PADDED_BANK_SIZE + CounterOffset] = 0;
		}
	}

	// Clear the raking counter padding.
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		[unroll]
		for ( i = 1; i <= RAKING_COUNTER_PADDING; ++i )
		{
			LocalRakingTotals[RakingIndex - i] = 0;
		}
	}

	// Clear LocalTotals padding.
	if ( ThreadId < DIGIT_SCAN_PADDING )
	{
		LocalTotals[ThreadId] = 0;
	}

	// Acquire LocalOffsets, LocalTotals, and LocalRakingTotals.
	GroupMemoryBarrierWithGroupSync();

	// Reduce.
	uint Total = 0;
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		// Perform a serial reduction on this raking thread's counters.
		Total = LocalOffsets[BankToRakeOffset];
		[unroll]
		for ( i = 1; i < COUNTERS_PER_RAKING_THREAD; ++i )
		{
			Total += LocalOffsets[BankToRakeOffset + i];
		}

		// Place the total in the raking counter.
		LocalRakingTotals[RakingIndex] = Total;
	}

	// Scan raking totals.
	{
		[unroll]
		for ( uint RakingOffset = 1; RakingOffset < RAKING_THREADS_PER_DIGIT; RakingOffset <<= 1 )
		{
			// Acquire LocalRakingTotals.
			PPS_BARRIER();
			Total = Total + LocalRakingTotals[RakingIndex - RakingOffset];
			LocalRakingTotals[RakingIndex] = Total;
		}
	}

	// Acquire LocalRakingTotals.
	GroupMemoryBarrierWithGroupSync();

	uint DigitSum = 0;
	if(ThreadId < DIGIT_COUNT)
	{
		DigitSum = LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * (ThreadId+1) - 1];
	}
	LocalTotals[ThreadId + DIGIT_SCAN_PADDING] = DigitSum;

	// Scan local totals.
	{
		[unroll]
		for ( uint RakingOffset = 1; RakingOffset < DIGIT_COUNT; RakingOffset <<= 1 )
		{
			// Acquire LocalTotals.
			PPS_BARRIER();
			DigitSum += LocalTotals[ThreadId + DIGIT_SCAN_PADDING - RakingOffset];
			GroupMemoryBarrierWithGroupSync();
			LocalTotals[ThreadId + DIGIT_SCAN_PADDING] = DigitSum;
		}
	}

	// Acquire LocalTotals.
	GroupMemoryBarrierWithGroupSync();

	// Serial scan.
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		uint Offset = LocalTotals[RakingDigit + DIGIT_SCAN_PADDING - 1] + LocalRakingTotals[RakingIndex - 1];
		[unroll]
		for ( i = 0; i < COUNTERS_PER_RAKING_THREAD; ++i )
		{
			uint Counter = LocalOffsets[BankToRakeOffset + i];
			LocalOffsets[BankToRakeOffset + i] = Offset;
			Offset += Counter;
		}
	}

	// Acquire LocalOffsets.
	GroupMemoryBarrierWithGroupSync();
	
	// Store offsets.
	if ( ThreadId < MAX_GROUP_COUNT )
	{
		for ( uint DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex )
		{
			const uint BankIndex = DigitIndex * BANKS_PER_DIGIT + BankOffset;
			OutOffsets[ThreadId * DIGIT_COUNT + DigitIndex] = LocalOffsets[BankIndex * PADDED_BANK_SIZE + CounterOffset];
		}
	}
}

#endif // #if RADIX_SORT_SPINE

/*------------------------------------------------------------------------------
	The downsweep sorting kernel. This kernel reads the per-work group partial
	sums in to LocalTotals. The kernel then recomputes much of the work done
	during the upsweep, this time computing a full set of prefix sums so that
	keys can be scattered in to global memory.
------------------------------------------------------------------------------*/

#if RADIX_SORT_DOWNSWEEP

/** The amount of padding to add for scanning digits. */
#define DIGIT_SCAN_PADDING (DIGIT_COUNT >> 1)

/** The amount of storage required to store counters. */
#define MAX_COUNTER_STORAGE (BANKS_PER_DIGIT * DIGIT_COUNT * PADDED_BANK_SIZE)

/** The maximum amount of storage required for temporary keys. */
#define MAX_KEY_STORAGE (THREAD_COUNT * KEYS_PER_LOOP)

/** The amount of scratch storage required, aliased by counters and keys. */
#define SCRATCH_STORAGE (MAX_COUNTER_STORAGE > MAX_KEY_STORAGE ? MAX_COUNTER_STORAGE : MAX_KEY_STORAGE)

/** Input keys to be sorted. */
Buffer<uint> InKeys;
/** Input values to be transferred with keys. */
Buffer<uint> InValues;
/** Input offsets. */
Buffer<uint> InOffsets;
/** Output buffer for sorted keys. */
RWBuffer<uint> OutKeys;
/** Output buffer for sorted values. */
RWBuffer<uint> OutValues;

/** Local scratch storage for scattering. Should be SCRATCH_STORAGE elements, but
	DirectCompute doesn't like the ternary operator being used for array sizes. */
groupshared uint LocalScratch[MAX_COUNTER_STORAGE];
/** Local storage for raking totals. */
groupshared uint LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * DIGIT_COUNT + 1];
/** Local storage for the per-digit totals. */
groupshared uint LocalTotals[128 + DIGIT_SCAN_PADDING];
/** Local storage for the input offsets. */
groupshared uint LocalOffsets[DIGIT_COUNT];

/**
 * Obtain digits for the set of keys.
 */
void GetDigits(
	uint Keys[KEYS_PER_LOOP],
	out uint Digits[KEYS_PER_LOOP] )
{
	uint KeyIndex;

	[unroll]
	for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
	{
		Digits[KeyIndex] = (Keys[KeyIndex] >> RadixSortUB.RadixShift) & DIGIT_MASK;
	}
}

/**
 * Scan keys to compute ranks. Ranks are stored in LocalScratch.
 */
void ScanKeys(
	const uint ThreadId,
	uint Digits[KEYS_PER_LOOP],
	const uint BankToRakeOffset,
	const uint RakingDigit,
	const uint RakingIndex,
	out uint DigitTotalThisTile )
{
	uint i;
	uint KeyIndex;
	uint DigitIndex;
	uint RakingOffset;

	DigitTotalThisTile = 0;
	const uint BankOffset = ThreadId / WORDS_PER_BANK;
	const uint CounterOffset = ThreadId & (WORDS_PER_BANK - 1);

	// Clear local counters.
	for ( DigitIndex = 0; DigitIndex < DIGIT_COUNT; ++DigitIndex )
	{
		const uint BankIndex = DigitIndex * BANKS_PER_DIGIT + BankOffset;
		LocalScratch[BankIndex * PADDED_BANK_SIZE + CounterOffset] = 0;
	}

	// Increment counters for each key.
	[unroll]
	for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
	{
		const uint BankIndex = Digits[KeyIndex] * BANKS_PER_DIGIT + BankOffset;
		LocalScratch[BankIndex * PADDED_BANK_SIZE + CounterOffset] += 1;
	}

	// Acquire counters.
	GroupMemoryBarrierWithGroupSync();

	// Reduce.
	uint Total = 0;
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		// Perform a serial reduction on this raking thread's counters.
		Total = LocalScratch[BankToRakeOffset];
		[unroll]
		for ( i = 1; i < COUNTERS_PER_RAKING_THREAD; ++i )
		{
			Total += LocalScratch[BankToRakeOffset + i];
		}

		// Place the total in the raking counter.
		LocalRakingTotals[RakingIndex] = Total;
	}

	// Scan raking totals.
	[unroll]
	for ( RakingOffset = 1; RakingOffset < RAKING_THREADS_PER_DIGIT; RakingOffset <<= 1 )
	{
		// Acquire LocalRakingTotals.
		PPS_BARRIER();
		Total += LocalRakingTotals[RakingIndex - RakingOffset];
		LocalRakingTotals[RakingIndex] = Total;
	}

	// Acquire LocalRakingTotals.
	GroupMemoryBarrierWithGroupSync();

	uint DigitSum = 0;
	if(ThreadId < DIGIT_COUNT)
	{
		DigitSum = LocalRakingTotals[RAKING_COUNTERS_PER_DIGIT * (ThreadId+1) - 1];
	}
	DigitTotalThisTile = DigitSum;
	LocalTotals[ThreadId + DIGIT_SCAN_PADDING] = DigitSum;

	// Scan local totals.
	[unroll]
	for ( RakingOffset = 1; RakingOffset < DIGIT_COUNT; RakingOffset <<= 1 )
	{
		// Acquire LocalTotals.
		PPS_BARRIER();
		DigitSum += LocalTotals[ThreadId + DIGIT_SCAN_PADDING - RakingOffset];
		GroupMemoryBarrierWithGroupSync();
		LocalTotals[ThreadId + DIGIT_SCAN_PADDING] = DigitSum;
	}

	// Acquire LocalTotals.
	GroupMemoryBarrierWithGroupSync();

	// Serial scan.
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		uint Offset = LocalTotals[RakingDigit + DIGIT_SCAN_PADDING - 1] + LocalRakingTotals[RakingIndex - 1];
		[unroll]
		for ( i = 0; i < COUNTERS_PER_RAKING_THREAD; ++i )
		{
			uint Counter = LocalScratch[BankToRakeOffset + i];
			LocalScratch[BankToRakeOffset + i] = Offset;
			Offset += Counter;
		}
	}

	// Acquire ranks in LocalScratch.
	GroupMemoryBarrierWithGroupSync();
}

/**
 * Compute offsets for the given digits.
 */
void ComputeOffsets(
	const uint ThreadId,
	uint Digits[KEYS_PER_LOOP],
	out uint KeyOffsets[KEYS_PER_LOOP] )
{
	uint KeyIndex;

	const uint BankOffset = ThreadId / WORDS_PER_BANK;
	const uint CounterOffset = ThreadId & (WORDS_PER_BANK - 1);

	// Compute key offsets.
	[unroll]
	for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
	{
		const uint BankIndex = Digits[KeyIndex] * BANKS_PER_DIGIT + BankOffset;
		const uint Offset = LocalScratch[BankIndex * PADDED_BANK_SIZE + CounterOffset];
		KeyOffsets[KeyIndex] = Offset;
		LocalScratch[BankIndex * PADDED_BANK_SIZE + CounterOffset] = Offset + 1;
	}

	// Release LocalScratch so we can scatter keys in to that memory.
	GroupMemoryBarrierWithGroupSync();
}

/**
 * Scatter to local memory.
 */
void ScatterLocal(
	const uint ThreadId,
	uint Values[KEYS_PER_LOOP],
	uint Offsets[KEYS_PER_LOOP] )
{
	uint KeyIndex;

	// Scatter keys to local memory.
	[unroll]
	for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
	{
		LocalScratch[Offsets[KeyIndex]] = Values[KeyIndex];
	}

	// Acquire values in LocalScratch.
	GroupMemoryBarrierWithGroupSync();
}

/**
 * Downsweep scan and scatter kernel.
 */
[numthreads(THREAD_COUNT,1,1)]
void RadixSort_Downsweep(
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupIdXYZ : SV_GroupID )
{
	uint Keys[KEYS_PER_LOOP];
	uint KeyOffsets[KEYS_PER_LOOP];
	uint Digits[KEYS_PER_LOOP];
	uint DigitTotalThisTile = 0;
	uint FirstTileIndex;
	uint TileCountForGroup;
	uint KeyIndex;
	uint i;

	// Determine global and local thread IDs.
	const uint ThreadId = GroupThreadId.x;
	const uint GroupId = GroupIdXYZ.x;

	const uint BankOffset = ThreadId / WORDS_PER_BANK;
	const uint CounterOffset = ThreadId & (WORDS_PER_BANK - 1);

	const uint BankToRakeOffset = GetBankToRakeOffset( ThreadId );
	const uint RakingDigit = GetRakingDigit( ThreadId );
	const uint RakingIndex = GetRakingIndex( ThreadId );

	// Clear the raking counter padding.
	if ( ThreadId < RAKING_THREAD_COUNT )
	{
		[unroll]
		for ( i = 1; i <= RAKING_COUNTER_PADDING; ++i )
		{
			LocalRakingTotals[RakingIndex - i] = 0;
		}
	}

	// Read work group prefix sums in to local memory.
	if ( ThreadId < DIGIT_COUNT )
	{
		LocalOffsets[ThreadId] = InOffsets[GroupId * DIGIT_COUNT + ThreadId];
	}

	// Clear the padding on local totals.
	if ( ThreadId < DIGIT_SCAN_PADDING )
	{
		LocalTotals[ThreadId] = 0;
	}

	// The number of tiles to process in this group.
	if ( GroupId < RadixSortUB.ExtraTileCount )
	{
		FirstTileIndex = GroupId * (RadixSortUB.TilesPerGroup + 1);
		TileCountForGroup = RadixSortUB.TilesPerGroup + 1;
	}
	else
	{
		FirstTileIndex = GroupId * RadixSortUB.TilesPerGroup + RadixSortUB.ExtraTileCount;
		TileCountForGroup = RadixSortUB.TilesPerGroup;
	}

	// Key range for this group.
	uint GroupKeyBegin = FirstTileIndex * TILE_SIZE;
	uint GroupKeyEnd = GroupKeyBegin + TileCountForGroup * TILE_SIZE;
	
	// Process each tile sequentially.
	while ( GroupKeyBegin < GroupKeyEnd )
	{
		// Load keys.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = InKeys[GroupKeyBegin + ThreadId * KEYS_PER_LOOP + KeyIndex];
		}

		// Scan keys and compute offsets.
		GetDigits( Keys, Digits );
		ScanKeys( ThreadId, Digits, BankToRakeOffset, RakingDigit, RakingIndex, DigitTotalThisTile );
		ComputeOffsets( ThreadId, Digits, KeyOffsets );

		// Scatter to local memory.
		ScatterLocal( ThreadId, Keys, KeyOffsets );
		
		// Obtain coalesced keys.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = LocalScratch[ThreadId * KEYS_PER_LOOP + KeyIndex];
		}
		GetDigits( Keys, Digits );

		// Scatter keys to global memory.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			const uint Digit = Digits[KeyIndex];
			const uint GlobalScatterIndex = LocalOffsets[Digit] + ThreadId * KEYS_PER_LOOP + KeyIndex - LocalTotals[Digit + DIGIT_SCAN_PADDING - 1];
			OutKeys[GlobalScatterIndex] = Keys[KeyIndex];
		}

		// Load values.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = InValues[GroupKeyBegin + ThreadId * KEYS_PER_LOOP + KeyIndex];
		}

		// Release LocalScratch so values can be scattered to local memory.
		GroupMemoryBarrierWithGroupSync();

		// Scatter to local memory.
		ScatterLocal( ThreadId, Keys, KeyOffsets );
		
		// Obtain coalesced values.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = LocalScratch[ThreadId * KEYS_PER_LOOP + KeyIndex];
		}

		// Scatter values to global memory.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			const uint Digit = Digits[KeyIndex];
			const uint GlobalScatterIndex = LocalOffsets[Digit] + ThreadId * KEYS_PER_LOOP + KeyIndex - LocalTotals[Digit + DIGIT_SCAN_PADDING - 1];
			OutValues[GlobalScatterIndex] = Keys[KeyIndex];
		}

		// Release LocalOffsets so it can be updated.
		GroupMemoryBarrierWithGroupSync();

		// Increment local offsets so the next tile has correct offsets.
		if ( ThreadId < DIGIT_COUNT )
		{
			LocalOffsets[ThreadId] += DigitTotalThisTile;
		}

		GroupKeyBegin += TILE_SIZE;
	}

	// The last group has to process any keys after the last tile.
	if ( GroupId == (RadixSortUB.GroupCount - 1) && RadixSortUB.ExtraKeyCount > 0 )
	{
		// Load keys.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			if ( ThreadId * KEYS_PER_LOOP + KeyIndex < RadixSortUB.ExtraKeyCount )
			{
				Keys[KeyIndex] = InKeys[GroupKeyBegin + ThreadId * KEYS_PER_LOOP + KeyIndex];
			}
			else
			{
				Keys[KeyIndex] = 0xFFFFFFFF;
			}
		}

		// Scan keys and compute offsets.
		GetDigits( Keys, Digits );
		ScanKeys( ThreadId, Digits, BankToRakeOffset, RakingDigit, RakingIndex, DigitTotalThisTile );
		ComputeOffsets( ThreadId, Digits, KeyOffsets );

		// Scatter to local memory.
		ScatterLocal( ThreadId, Keys, KeyOffsets );
		
		// Obtain coalesced keys.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = LocalScratch[ThreadId * KEYS_PER_LOOP + KeyIndex];
		}
		GetDigits( Keys, Digits );

		// Scatter keys to global memory.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			if ( ThreadId * KEYS_PER_LOOP + KeyIndex < RadixSortUB.ExtraKeyCount )
			{
				const uint Digit = Digits[KeyIndex];
				const uint GlobalScatterIndex = LocalOffsets[Digit] + ThreadId * KEYS_PER_LOOP + KeyIndex - LocalTotals[Digit + DIGIT_SCAN_PADDING - 1];
				OutKeys[GlobalScatterIndex] = Keys[KeyIndex];
			}
		}

		// Load values.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			if ( ThreadId * KEYS_PER_LOOP + KeyIndex < RadixSortUB.ExtraKeyCount )
			{
				Keys[KeyIndex] = InValues[GroupKeyBegin + ThreadId * KEYS_PER_LOOP + KeyIndex];
			}
			else
			{
				Keys[KeyIndex] = 0x0BADF00D;
			}
		}

		// Release LocalScratch so values can be scattered to local memory.
		GroupMemoryBarrierWithGroupSync();

		// Scatter to local memory.
		ScatterLocal( ThreadId, Keys, KeyOffsets );
		
		// Obtain coalesced values.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			Keys[KeyIndex] = LocalScratch[ThreadId * KEYS_PER_LOOP + KeyIndex];
		}

		// Scatter values to global memory.
		[unroll]
		for ( KeyIndex = 0; KeyIndex < KEYS_PER_LOOP; ++KeyIndex )
		{
			if ( ThreadId * KEYS_PER_LOOP + KeyIndex < RadixSortUB.ExtraKeyCount )
			{
				const uint Digit = Digits[KeyIndex];
				const uint GlobalScatterIndex = LocalOffsets[Digit] + ThreadId * KEYS_PER_LOOP + KeyIndex - LocalTotals[Digit + DIGIT_SCAN_PADDING - 1];
				OutValues[GlobalScatterIndex] = Keys[KeyIndex];
			}
		}
	}
}

#endif // #if RADIX_SORT_DOWNSWEEP
