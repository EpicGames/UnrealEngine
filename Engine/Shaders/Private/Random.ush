// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Random.ush: Pseudo-random number generators.
=============================================================================*/

#pragma once

// Most frequently used functions in separate smaller headers
#include "RandomInterleavedGradientNoise.ush"
#include "RandomPCG.ush"

// @param xy should be a integer position (e.g. pixel position on the screen), repeats each 128x128 pixels
// similar to a texture lookup but is only ALU
// ~13 ALU operations (3 frac, 6 *, 4 mad)
float PseudoRandom(float2 xy)
{
	float2 pos = frac(xy / 128.0f) * 128.0f + float2(-64.340622f, -72.465622f);
	
	// found by experimentation
	return frac(dot(pos.xyx * pos.xyy, float3(20.390625f, 60.703125f, 2.4281209f)));
}

// [0, 1[
// ~10 ALU operations (2 frac, 5 *, 3 mad)
float RandFast( uint2 PixelPos, float Magic = 3571.0 )
{
	float2 Random2 = ( 1.0 / 4320.0 ) * PixelPos + float2( 0.25, 0.0 );
	float Random = frac( dot( Random2 * Random2, Magic ) );
	Random = frac( Random * Random * (2 * Magic) );
	return Random;
}

// This is the largest prime < 2^12 so s*s will fit in a 24-bit floating point mantissa
#define BBS_PRIME24 4093

// Blum-Blum-Shub-inspired pseudo random number generator
// http://www.umbc.edu/~olano/papers/mNoise.pdf
// real BBS uses ((s*s) mod M) with bignums and M as the product of two huge Blum primes
// instead, we use a single prime M just small enough not to overflow
// note that the above paper used 61, which fits in a half, but is unusably bad
// @param Integer valued floating point seed
// @return random number in range [0,1)
// ~8 ALU operations (5 *, 3 frac)
float RandBBSfloat(float seed)
{
	float s = frac(seed / BBS_PRIME24);
	s = frac(s * s * BBS_PRIME24);
	s = frac(s * s * BBS_PRIME24);
	return s;
}

/**
 * Find good arbitrary axis vectors to represent U and V axes of a plane,
 * given just the normal. Ported from UnMath.h
 */
void FindBestAxisVectors(float3 In, out float3 Axis1, out float3 Axis2 )
{
	const float3 N = abs(In);

	// Find best basis vectors.
	if( N.z > N.x && N.z > N.y )
	{
		Axis1 = float3(1, 0, 0);
	}
	else
	{
		Axis1 = float3(0, 0, 1);
	}

	Axis1 = normalize(Axis1 - In * dot(Axis1, In));
	Axis2 = cross(Axis1, In);
}

// References for noise:
//
// Improved Perlin noise
//   http://mrl.nyu.edu/~perlin/noise/
//   http://http.developer.nvidia.com/GPUGems/gpugems_ch05.html
// Modified Noise for Evaluation on Graphics Hardware
//   http://www.csee.umbc.edu/~olano/papers/mNoise.pdf
// Perlin Noise
//   http://mrl.nyu.edu/~perlin/doc/oscar.html
// Fast Gradient Noise
//   http://prettyprocs.wordpress.com/2012/10/20/fast-perlin-noise


// -------- ALU based method ---------

/*
 * Pseudo random number generator, based on "TEA, a tiny Encrytion Algorithm"
 * http://citeseer.ist.psu.edu/viewdoc/download?doi=10.1.1.45.281&rep=rep1&type=pdf
 * http://www.umbc.edu/~olano/papers/index.html#GPUTEA
 * @param v - old seed (full 32bit range)
 * @param IterationCount - >=1, bigger numbers cost more performance but improve quality
 * @return new seed
 */
uint2 ScrambleTEA(uint2 v, uint IterationCount = 3)
{
	// Start with some random data (numbers can be arbitrary but those have been used by others and seem to work well)
	uint k[4] ={ 0xA341316Cu , 0xC8013EA4u , 0xAD90777Du , 0x7E95761Eu };
	
	uint y = v[0];
	uint z = v[1];
	uint sum = 0;
	
	UNROLL for(uint i = 0; i < IterationCount; ++i)
	{
		sum += 0x9e3779b9;
		y += ((z << 4u) + k[0]) ^ (z + sum) ^ ((z >> 5u) + k[1]);
		z += ((y << 4u) + k[2]) ^ (y + sum) ^ ((y >> 5u) + k[3]);
	}

	return uint2(y, z);
}

// Wraps noise for tiling texture creation
// @param v = unwrapped texture parameter
// @param bTiling = true to tile, false to not tile
// @param RepeatSize = number of units before repeating
// @return either original or wrapped coord
float3 NoiseTileWrap(float3 v,  bool bTiling, float RepeatSize)
{
	return bTiling ? (frac(v / RepeatSize) * RepeatSize) : v;
}

// Evaluate polynomial to get smooth transitions for Perlin noise
// only needed by Perlin functions in this file
// scalar(per component): 2 add, 5 mul
float4 PerlinRamp(float4 t)
{
	return t * t * t * (t * (t * 6 - 15) + 10); 
}

// Analytical derivative of the PerlinRamp polynomial
// only needed by Perlin functions in this file
// scalar(per component): 2 add, 5 mul
float4 PerlinRampDerivative(float4 t)
{
	return t * t * (t * (t * 30 - 60) + 30);
}

#define MGradientMask int3(0x8000, 0x4000, 0x2000)
#define MGradientScale float3(1. / 0x4000, 1. / 0x2000, 1. / 0x1000)
// Modified noise gradient term
// @param seed - random seed for integer lattice position
// @param offset - [-1,1] offset of evaluation point from lattice point
// @return gradient direction (xyz) and contribution (w) from this lattice point
float4 MGradient(int seed, float3 offset)
{
	uint rand = Rand3DPCG16(int3(seed,0,0)).x;
	float3 direction = float3(rand.xxx & MGradientMask) * MGradientScale - 1;
	return float4(direction, dot(direction, offset));
}

// compute Perlin and related noise corner seed values
// @param v = 3D noise argument, use float3(x,y,0) for 2D or float3(x,0,0) for 1D
// @param bTiling = true to return seed values for a repeating noise pattern
// @param RepeatSize = integer units before tiling in each dimension
// @param seed000-seed111 = hash function seeds for the eight corners
// @return fractional part of v
float3 NoiseSeeds(float3 v, bool bTiling, float RepeatSize,
	out float seed000, out float seed001, out float seed010, out float seed011,
	out float seed100, out float seed101, out float seed110, out float seed111)
{
	float3 fv = frac(v);
	float3 iv = floor(v);

	const float3 primes = float3(19, 47, 101);

	if (bTiling)
	{	// can't algebraically combine with primes
		seed000 = dot(primes, NoiseTileWrap(iv, true, RepeatSize));
		seed100 = dot(primes, NoiseTileWrap(iv + float3(1, 0, 0), true, RepeatSize));
		seed010 = dot(primes, NoiseTileWrap(iv + float3(0, 1, 0), true, RepeatSize));
		seed110 = dot(primes, NoiseTileWrap(iv + float3(1, 1, 0), true, RepeatSize));
		seed001 = dot(primes, NoiseTileWrap(iv + float3(0, 0, 1), true, RepeatSize));
		seed101 = dot(primes, NoiseTileWrap(iv + float3(1, 0, 1), true, RepeatSize));
		seed011 = dot(primes, NoiseTileWrap(iv + float3(0, 1, 1), true, RepeatSize));
		seed111 = dot(primes, NoiseTileWrap(iv + float3(1, 1, 1), true, RepeatSize));
	}
	else
	{	// get to combine offsets with multiplication by primes in this case
		seed000 = dot(iv, primes);
		seed100 = seed000 + primes.x;
		seed010 = seed000 + primes.y;
		seed110 = seed100 + primes.y;
		seed001 = seed000 + primes.z;
		seed101 = seed100 + primes.z;
		seed011 = seed010 + primes.z;
		seed111 = seed110 + primes.z;
	}

	return fv;
}

// Perlin-style "Modified Noise"
// http://www.umbc.edu/~olano/papers/index.html#mNoise
// @param v = 3D noise argument, use float3(x,y,0) for 2D or float3(x,0,0) for 1D
// @param bTiling = repeat noise pattern
// @param RepeatSize = integer units before tiling in each dimension
// @return random number in the range -1 .. 1
float GradientNoise3D_ALU(float3 v, bool bTiling, float RepeatSize)
{
	float seed000, seed001, seed010, seed011, seed100, seed101, seed110, seed111;
	float3 fv = NoiseSeeds(v, bTiling, RepeatSize, seed000, seed001, seed010, seed011, seed100, seed101, seed110, seed111);

	float rand000 = MGradient(int(seed000), fv - float3(0, 0, 0)).w;
	float rand100 = MGradient(int(seed100), fv - float3(1, 0, 0)).w;
	float rand010 = MGradient(int(seed010), fv - float3(0, 1, 0)).w;
	float rand110 = MGradient(int(seed110), fv - float3(1, 1, 0)).w;
	float rand001 = MGradient(int(seed001), fv - float3(0, 0, 1)).w;
	float rand101 = MGradient(int(seed101), fv - float3(1, 0, 1)).w;
	float rand011 = MGradient(int(seed011), fv - float3(0, 1, 1)).w;
	float rand111 = MGradient(int(seed111), fv - float3(1, 1, 1)).w;

	float3 Weights = PerlinRamp(float4(fv, 0)).xyz;

	float i = lerp(lerp(rand000, rand100, Weights.x), lerp(rand010, rand110, Weights.x), Weights.y);
	float j = lerp(lerp(rand001, rand101, Weights.x), lerp(rand011, rand111, Weights.x), Weights.y);
	return lerp(i, j, Weights.z).x;
}

// Coordinates for corners of a Simplex tetrahedron
// Based on McEwan et al., Efficient computation of noise in GLSL, JGT 2011
// @param v = 3D noise argument
// @return 4 corner locations
float4x3 SimplexCorners(float3 v)
{
	// find base corner by skewing to tetrahedral space and back
	float3 tet = floor(v + v.x/3 + v.y/3 + v.z/3);
	float3 base = tet - tet.x/6 - tet.y/6 - tet.z/6;
	float3 f = v - base;

	// Find offsets to other corners (McEwan did this in tetrahedral space,
	// but since skew is along x=y=z axis, this works in Euclidean space too.)
	float3 g = step(f.yzx, f.xyz), h = 1 - g.zxy;
	float3 a1 = min(g, h) - 1. / 6., a2 = max(g, h) - 1. / 3.;

	// four corners
	return float4x3(base, base + a1, base + a2, base + 0.5);
}

// Improved smoothing function for simplex noise
// @param f = fractional distance to four tetrahedral corners
// @return weight for each corner
float4 SimplexSmooth(float4x3 f)
{
	const float scale = 1024. / 375.;	// scale factor to make noise -1..1
	float4 d = float4(dot(f[0], f[0]), dot(f[1], f[1]), dot(f[2], f[2]), dot(f[3], f[3]));
	float4 s = saturate(2 * d);
	return (1 * scale + s*(-3 * scale + s*(3 * scale - s*scale)));
}

// Derivative of simplex noise smoothing function
// @param f = fractional distanc eto four tetrahedral corners
// @return derivative of smoothing function for each corner by x, y and z
float3x4 SimplexDSmooth(float4x3 f)
{
	const float scale = 1024. / 375.;	// scale factor to make noise -1..1
	float4 d = float4(dot(f[0], f[0]), dot(f[1], f[1]), dot(f[2], f[2]), dot(f[3], f[3]));
	float4 s = saturate(2 * d);
	s = -12 * scale + s*(24 * scale - s * 12 * scale);

	return float3x4(
		s * float4(f[0][0], f[1][0], f[2][0], f[3][0]),
		s * float4(f[0][1], f[1][1], f[2][1], f[3][1]),
		s * float4(f[0][2], f[1][2], f[2][2], f[3][2]));
}

// Simplex noise and its Jacobian derivative
// @param v = 3D noise argument
// @param bTiling = whether to repeat noise pattern
// @param RepeatSize = integer units before tiling in each dimension, must be a multiple of 3
// @return float3x3 Jacobian in J[*].xyz, vector noise in J[*].w
//     J[0].w, J[1].w, J[2].w is a Perlin-style simplex noise with vector output, e.g. (Nx, Ny, Nz)
//     J[i].x is X derivative of the i'th component of the noise so J[2].x is dNz/dx
// You can use this to compute the noise, gradient, curl, or divergence:
//   float3x4 J = JacobianSimplex_ALU(...);
//   float3 VNoise = float3(J[0].w, J[1].w, J[2].w);	// 3D noise
//   float3 Grad = J[0].xyz;							// gradient of J[0].w
//   float3 Curl = float3(J[1][2]-J[2][1], J[2][0]-J[0][2], J[0][1]-J[1][2]);
//   float Div = J[0][0]+J[1][1]+J[2][2];
// All of these are confirmed to compile out all unneeded terms.
// So Grad of X doesn't compute Y or Z components, and VNoise doesn't do any of the derivative computation.
float3x4 JacobianSimplex_ALU(float3 v, bool bTiling, float RepeatSize)
{
	// corners of tetrahedron
	float4x3 T = SimplexCorners(v);
	uint3 rand;
	float4x3 gvec[3], fv;
	float3x4 grad;

	// processing of tetrahedral vertices, unrolled
	// to compute gradient at each corner
	fv[0] = v - T[0];
	rand = Rand3DPCG16(int3(floor(NoiseTileWrap(6 * T[0] + 0.5, bTiling, RepeatSize))));
	gvec[0][0] = float3(rand.xxx & MGradientMask) * MGradientScale - 1;
	gvec[1][0] = float3(rand.yyy & MGradientMask) * MGradientScale - 1;
	gvec[2][0] = float3(rand.zzz & MGradientMask) * MGradientScale - 1;
	grad[0][0] = dot(gvec[0][0], fv[0]);
	grad[1][0] = dot(gvec[1][0], fv[0]);
	grad[2][0] = dot(gvec[2][0], fv[0]);

	fv[1] = v - T[1];
	rand = Rand3DPCG16(int3(floor(NoiseTileWrap(6 * T[1] + 0.5, bTiling, RepeatSize))));
	gvec[0][1] = float3(rand.xxx & MGradientMask) * MGradientScale - 1;
	gvec[1][1] = float3(rand.yyy & MGradientMask) * MGradientScale - 1;
	gvec[2][1] = float3(rand.zzz & MGradientMask) * MGradientScale - 1;
	grad[0][1] = dot(gvec[0][1], fv[1]);
	grad[1][1] = dot(gvec[1][1], fv[1]);
	grad[2][1] = dot(gvec[2][1], fv[1]);

	fv[2] = v - T[2];
	rand = Rand3DPCG16(int3(floor(NoiseTileWrap(6 * T[2] + 0.5, bTiling, RepeatSize))));
	gvec[0][2] = float3(rand.xxx & MGradientMask) * MGradientScale - 1;
	gvec[1][2] = float3(rand.yyy & MGradientMask) * MGradientScale - 1;
	gvec[2][2] = float3(rand.zzz & MGradientMask) * MGradientScale - 1;
	grad[0][2] = dot(gvec[0][2], fv[2]);
	grad[1][2] = dot(gvec[1][2], fv[2]);
	grad[2][2] = dot(gvec[2][2], fv[2]);

	fv[3] = v - T[3];
	rand = Rand3DPCG16(int3(floor(NoiseTileWrap(6 * T[3] + 0.5, bTiling, RepeatSize))));
	gvec[0][3] = float3(rand.xxx & MGradientMask) * MGradientScale - 1;
	gvec[1][3] = float3(rand.yyy & MGradientMask) * MGradientScale - 1;
	gvec[2][3] = float3(rand.zzz & MGradientMask) * MGradientScale - 1;
	grad[0][3] = dot(gvec[0][3], fv[3]);
	grad[1][3] = dot(gvec[1][3], fv[3]);
	grad[2][3] = dot(gvec[2][3], fv[3]);

	// blend gradients
	float4 sv = SimplexSmooth(fv);
	float3x4 ds = SimplexDSmooth(fv);

	float3x4 jacobian;
	jacobian[0] = float4(mul(sv, gvec[0]) + mul(ds, grad[0]), dot(sv, grad[0]));
	jacobian[1] = float4(mul(sv, gvec[1]) + mul(ds, grad[1]), dot(sv, grad[1]));
	jacobian[2] = float4(mul(sv, gvec[2]) + mul(ds, grad[2]), dot(sv, grad[2]));

	return jacobian;
}

// 3D value noise - used to be incorrectly called Perlin noise
// @param v = 3D noise argument, use float3(x,y,0) for 2D or float3(x,0,0) for 1D
// @param bTiling = repeat noise pattern
// @param RepeatSize = integer units before tiling in each dimension
// @return random number in the range -1 .. 1
float ValueNoise3D_ALU(float3 v, bool bTiling, float RepeatSize)
{
	float seed000, seed001, seed010, seed011, seed100, seed101, seed110, seed111;
	float3 fv = NoiseSeeds(v, bTiling, RepeatSize, seed000, seed001, seed010, seed011, seed100, seed101, seed110, seed111);

	float rand000 = RandBBSfloat(seed000) * 2 - 1;
	float rand100 = RandBBSfloat(seed100) * 2 - 1;
	float rand010 = RandBBSfloat(seed010) * 2 - 1;
	float rand110 = RandBBSfloat(seed110) * 2 - 1;
	float rand001 = RandBBSfloat(seed001) * 2 - 1;
	float rand101 = RandBBSfloat(seed101) * 2 - 1;
	float rand011 = RandBBSfloat(seed011) * 2 - 1;
	float rand111 = RandBBSfloat(seed111) * 2 - 1;
	
	float3 Weights = PerlinRamp(float4(fv, 0)).xyz;
	
	float i = lerp(lerp(rand000, rand100, Weights.x), lerp(rand010, rand110, Weights.x), Weights.y);
	float j = lerp(lerp(rand001, rand101, Weights.x), lerp(rand011, rand111, Weights.x), Weights.y);
	return lerp(i, j, Weights.z).x;
}


// -------- TEX based methods ---------

// filtered 3D noise, can be optimized
// @param v = 3D noise argument, use float3(x,y,0) for 2D or float3(x,0,0) for 1D
// @param bTiling = repeat noise pattern
// @param RepeatSize = integer units before tiling in each dimension
// @return random number in the range -1 .. 1
float GradientNoise3D_TEX(float3 v, bool bTiling, float RepeatSize)
{
	bTiling = true;
	float3 fv = frac(v);
	float3 iv0 = NoiseTileWrap(floor(v), bTiling, RepeatSize);
	float3 iv1 = NoiseTileWrap(iv0 + 1, bTiling, RepeatSize);

	const int2 ZShear = int2(17, 89);
	
	float2 OffsetA = iv0.z * ZShear;
	float2 OffsetB = OffsetA + ZShear;	// non-tiling, use relative offset
	if (bTiling)						// tiling, have to compute from wrapped coordinates
	{
		OffsetB = iv1.z * ZShear;
	}

	// Texture size scale factor
	float ts = 1 / 128.0f;

	// texture coordinates for iv0.xy, as offset for both z slices
	float2 TexA0 = (iv0.xy + OffsetA + 0.5f) * ts;
	float2 TexB0 = (iv0.xy + OffsetB + 0.5f) * ts;

	// texture coordinates for iv1.xy, as offset for both z slices
	float2 TexA1 = TexA0 + ts;	// for non-tiling, can compute relative to existing coordinates
	float2 TexB1 = TexB0 + ts;
	if (bTiling)				// for tiling, need to compute from wrapped coordinates
	{
		TexA1 = (iv1.xy + OffsetA + 0.5f) * ts;
		TexB1 = (iv1.xy + OffsetB + 0.5f) * ts;
	}


	// can be optimized to 1 or 2 texture lookups (4 or 8 channel encoded in 8, 16 or 32 bit)
	float3 A = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexA0.x, TexA0.y), 0).xyz * 2 - 1;
	float3 B = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexA1.x, TexA0.y), 0).xyz * 2 - 1;
	float3 C = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexA0.x, TexA1.y), 0).xyz * 2 - 1;
	float3 D = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexA1.x, TexA1.y), 0).xyz * 2 - 1;
	float3 E = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexB0.x, TexB0.y), 0).xyz * 2 - 1;
	float3 F = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexB1.x, TexB0.y), 0).xyz * 2 - 1;
	float3 G = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexB0.x, TexB1.y), 0).xyz * 2 - 1;
	float3 H = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, float2(TexB1.x, TexB1.y), 0).xyz * 2 - 1;

	float a = dot(A, fv - float3(0, 0, 0));
	float b = dot(B, fv - float3(1, 0, 0));
	float c = dot(C, fv - float3(0, 1, 0));
	float d = dot(D, fv - float3(1, 1, 0));
	float e = dot(E, fv - float3(0, 0, 1));
	float f = dot(F, fv - float3(1, 0, 1));
	float g = dot(G, fv - float3(0, 1, 1));
	float h = dot(H, fv - float3(1, 1, 1));

	float3 Weights = PerlinRamp(frac(float4(fv, 0))).xyz;
	
	float i = lerp(lerp(a, b, Weights.x), lerp(c, d, Weights.x), Weights.y);
	float j = lerp(lerp(e, f, Weights.x), lerp(g, h, Weights.x), Weights.y);

	return lerp(i, j, Weights.z);
}

// @return random number in the range -1 .. 1
// scalar: 6 frac, 31 mul/mad, 15 add, 
float FastGradientPerlinNoise3D_TEX(float3 xyz)
{
	// needs to be the same value when creating the PerlinNoise3D texture
	float Extent = 16;

	// last texel replicated and needed for filtering
	// scalar: 3 frac, 6 mul
	xyz = frac(xyz / (Extent - 1)) * (Extent - 1);

	// scalar: 3 frac
	float3 uvw = frac(xyz);
	// = floor(xyz);
	// scalar: 3 add
	float3 p0 = xyz - uvw;
//	float3 f = pow(uvw, 2) * 3.0f - pow(uvw, 3) * 2.0f;	// original perlin hermite (ok when used without bump mapping)
	// scalar: 2*3 add 5*3 mul
	float3 f = PerlinRamp(float4(uvw, 0)).xyz;	// new, better with continues second derivative for bump mapping
	// scalar: 3 add
	float3 p = p0 + f;
	// scalar: 3 mad
	float4 NoiseSample = Texture3DSampleLevel(View.PerlinNoise3DTexture, View.PerlinNoise3DTextureSampler, p / Extent + 0.5f / Extent, 0);		// +0.5f to get rid of bilinear offset

	// reconstruct from 8bit (using mad with 2 constants and dot4 was same instruction count)
	// scalar: 4 mad, 3 mul, 3 add 
	float3 n = NoiseSample.xyz * 255.0f / 127.0f - 1.0f;
	float d = NoiseSample.w * 255.f - 127;
	return dot(xyz, n) - d;
}


// 3D jitter offset within a voronoi noise cell
// @param pos - integer lattice corner
// @return random offsets vector
float3 VoronoiCornerSample(float3 pos, int Quality)
{
	// random values in [-0.5, 0.5]
	float3 noise = float3(Rand3DPCG16(int3(pos))) / 0xffff - 0.5;

	// quality level 1 or 2: searches a 2x2x2 neighborhood with points distributed on a sphere
	// scale factor to guarantee jittered points will be found within a 2x2x2 search
	if (Quality <= 2)
	{
		return normalize(noise) * 0.2588;
	}

	// quality level 3: searches a 3x3x3 neighborhood with points distributed on a sphere
	// scale factor to guarantee jittered points will be found within a 3x3x3 search
	if (Quality == 3)
	{
		return normalize(noise) * 0.3090;
	}

	// quality level 4: jitter to anywhere in the cell, needs 4x4x4 search
	return noise;
}

// compare previous best with a new candidate
// not producing point locations makes it easier for compiler to eliminate calculations when they're not needed
// @param minval = location and distance of best candidate seed point before the new one
// @param candidate = candidate seed point
// @param offset = 3D offset to new candidate seed point
// @param bDistanceOnly = if true, only set maxval.w with distance, otherwise maxval.w is distance and maxval.xyz is position
// @return position (if bDistanceOnly is false) and distance to closest seed point so far
float4 VoronoiCompare(float4 minval, float3 candidate, float3 offset, bool bDistanceOnly)
{
	if (bDistanceOnly)
	{
		return float4(0, 0, 0, min(minval.w, dot(offset, offset)));
	}
	else
	{
		float newdist = dot(offset, offset);
		return newdist > minval.w ? minval : float4(candidate, newdist);
	}
}

// 220 instruction Worley noise
float4 VoronoiNoise3D_ALU(float3 v, int Quality, bool bTiling, float RepeatSize, bool bDistanceOnly)
{
	float3 fv = frac(v),  fv2 = frac(v + 0.5);
	float3 iv = floor(v), iv2 = floor(v + 0.5);

	// with initial minimum distance = infinity (or at least bigger than 4), first min is optimized away
	float4 mindist = float4(0,0,0,100);
	float3 p, offset;

	// quality level 3: do a 3x3x3 search
	if (Quality == 3)
	{
		UNROLL_N(3) for (offset.x = -1; offset.x <= 1; ++offset.x)
		{
			UNROLL_N(3) for (offset.y = -1; offset.y <= 1; ++offset.y)
			{
				UNROLL_N(3) for (offset.z = -1; offset.z <= 1; ++offset.z)
				{
					p = offset + VoronoiCornerSample(NoiseTileWrap(iv2 + offset, bTiling, RepeatSize), Quality);
					mindist = VoronoiCompare(mindist, iv2 + p, fv2 - p, bDistanceOnly);
				}
			}
		}
	}

	// everybody else searches a base 2x2x2 neighborhood
	else
	{
		UNROLL_N(2) for (offset.x = 0; offset.x <= 1; ++offset.x)
		{
			UNROLL_N(2) for (offset.y = 0; offset.y <= 1; ++offset.y)
			{
				UNROLL_N(2) for (offset.z = 0; offset.z <= 1; ++offset.z)
				{
					p = offset + VoronoiCornerSample(NoiseTileWrap(iv + offset, bTiling, RepeatSize), Quality);
					mindist = VoronoiCompare(mindist, iv + p, fv - p, bDistanceOnly);

					// quality level 2, do extra set of points, offset by half a cell
					if (Quality == 2)
					{
						// 467 is just an offset to a different area in the random number field to avoid similar neighbor artifacts
						p = offset + VoronoiCornerSample(NoiseTileWrap(iv2 + offset, bTiling, RepeatSize) + 467, Quality);
						mindist = VoronoiCompare(mindist, iv2 + p, fv2 - p, bDistanceOnly);
					}
				}
			}
		}
	}

	// quality level 4: add extra sets of four cells in each direction
	if (Quality >= 4)
	{
		UNROLL_N(2) for (offset.x = -1; offset.x <= 2; offset.x += 3)
		{
			UNROLL_N(2) for (offset.y = 0; offset.y <= 1; ++offset.y)
			{
				UNROLL_N(2) for (offset.z = 0; offset.z <= 1; ++offset.z)
				{
					// along x axis
					p = offset.xyz + VoronoiCornerSample(NoiseTileWrap(iv + offset.xyz, bTiling, RepeatSize), Quality);
					mindist = VoronoiCompare(mindist, iv + p, fv - p, bDistanceOnly);

					// along y axis
					p = offset.yzx + VoronoiCornerSample(NoiseTileWrap(iv + offset.yzx, bTiling, RepeatSize), Quality);
					mindist = VoronoiCompare(mindist, iv + p, fv - p, bDistanceOnly);

					// along z axis
					p = offset.zxy + VoronoiCornerSample(NoiseTileWrap(iv + offset.zxy, bTiling, RepeatSize), Quality);
					mindist = VoronoiCompare(mindist, iv + p, fv - p, bDistanceOnly);
				}
			}
		}
	}

	// transform squared distance to real distance
	return float4(mindist.xyz, sqrt(mindist.w));
}


// -------- Simplex method (faster in higher dimensions because less samples are used, uses gradient noise for quality) ---------
// <Dimensions>D:<Normal>/<Simplex> 1D:2, 2D:4/3, 3D:8/4, 4D:16/5 

// Computed weights and sample positions for simplex interpolation
// @return float3(a,b,c) Barycentric coordianate defined as Filtered = Tex(PosA) * a + Tex(PosB) * b + Tex(PosC) * c
float3 ComputeSimplexWeights2D(float2 OrthogonalPos, out float2 PosA, out float2 PosB, out float2 PosC)
{
	float2 OrthogonalPosFloor = floor(OrthogonalPos); 
	PosA = OrthogonalPosFloor;
	PosB = PosA + float2(1, 1); 

	float2 LocalPos = OrthogonalPos - OrthogonalPosFloor;

	PosC = PosA + ((LocalPos.x > LocalPos.y) ? float2(1,0) : float2(0,1));

	float b = min(LocalPos.x, LocalPos.y);
	float c = abs(LocalPos.y - LocalPos.x);
	float a = 1.0f - b - c;

	return float3(a, b, c);
}

// Computed weights and sample positions for simplex interpolation
// @return float4(a,b,c, d) Barycentric coordinate defined as Filtered = Tex(PosA) * a + Tex(PosB) * b + Tex(PosC) * c + Tex(PosD) * d
float4 ComputeSimplexWeights3D(float3 OrthogonalPos, out float3 PosA, out float3 PosB, out float3 PosC, out float3 PosD)
{
	float3 OrthogonalPosFloor = floor(OrthogonalPos);

	PosA = OrthogonalPosFloor;
	PosB = PosA + float3(1, 1, 1);

	OrthogonalPos -= OrthogonalPosFloor;

	float Largest = max(OrthogonalPos.x, max(OrthogonalPos.y, OrthogonalPos.z));
	float Smallest = min(OrthogonalPos.x, min(OrthogonalPos.y, OrthogonalPos.z));

	PosC = PosA + float3(Largest == OrthogonalPos.x, Largest == OrthogonalPos.y, Largest == OrthogonalPos.z);
	PosD = PosA + float3(Smallest != OrthogonalPos.x, Smallest != OrthogonalPos.y, Smallest != OrthogonalPos.z);

	float4 ret;

	float RG = OrthogonalPos.x - OrthogonalPos.y;
	float RB = OrthogonalPos.x - OrthogonalPos.z;
	float GB = OrthogonalPos.y - OrthogonalPos.z;

	ret.b = 
		  min(max(0, RG), max(0, RB))		// X
		+ min(max(0, -RG), max(0, GB))		// Y
		+ min(max(0, -RB), max(0, -GB));	// Z
	
	ret.a = 
		  min(max(0, -RG), max(0, -RB))		// X
		+ min(max(0, RG), max(0, -GB))		// Y
		+ min(max(0, RB), max(0, GB));		// Z

	ret.g = Smallest;
	ret.r = 1.0f - ret.g - ret.b - ret.a;

	return ret;
}

float2 GetPerlinNoiseGradientTextureAt(float2 v)
{
	float2 TexA = (v.xy + 0.5f) / 128.0f;

	// todo: storing random 2d unit vectors would be better
	float3 p = Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, TexA, 0).xyz * 2 - 1;
	return normalize(p.xy + p.z * 0.33f);
}

float3 GetPerlinNoiseGradientTextureAt(float3 v)
{
	const float2 ZShear = float2(17.0f, 89.0f);

	float2 OffsetA = v.z * ZShear;
	float2 TexA = (v.xy + OffsetA + 0.5f) / 128.0f;

	return Texture2DSampleLevel(View.PerlinNoiseGradientTexture, View.PerlinNoiseGradientTextureSampler, TexA , 0).xyz * 2 - 1;
}

float2 SkewSimplex(float2 In)
{
	return In + dot(In, (sqrt(3.0f) - 1.0f) * 0.5f );
}
float2 UnSkewSimplex(float2 In)
{
	return In - dot(In, (3.0f - sqrt(3.0f)) / 6.0f );
}
float3 SkewSimplex(float3 In)
{
	return In + dot(In, 1.0 / 3.0f );
}
float3 UnSkewSimplex(float3 In)
{
	return In - dot(In, 1.0 / 6.0f );
}

// filtered 3D gradient simple noise (few texture lookups, high quality)
// @param v >0
// @return random number in the range -1 .. 1
float GradientSimplexNoise2D_TEX(float2 EvalPos)
{
	float2 OrthogonalPos = SkewSimplex(EvalPos);

	float2 PosA, PosB, PosC, PosD;
	float3 Weights = ComputeSimplexWeights2D(OrthogonalPos, PosA, PosB, PosC);

	// can be optimized to 1 or 2 texture lookups (4 or 8 channel encoded in 32 bit)
	float2 A = GetPerlinNoiseGradientTextureAt(PosA);
	float2 B = GetPerlinNoiseGradientTextureAt(PosB);
	float2 C = GetPerlinNoiseGradientTextureAt(PosC);

	PosA = UnSkewSimplex(PosA);
	PosB = UnSkewSimplex(PosB);
	PosC = UnSkewSimplex(PosC);

	float DistanceWeight;

	DistanceWeight = saturate(0.5f - length2(EvalPos - PosA));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float a = dot(A, EvalPos - PosA) * DistanceWeight;
	DistanceWeight = saturate(0.5f - length2(EvalPos - PosB));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float b = dot(B, EvalPos - PosB) * DistanceWeight;
	DistanceWeight = saturate(0.5f - length2(EvalPos - PosC));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float c = dot(C, EvalPos - PosC) * DistanceWeight;
	
	return 70 * (a + b + c);
}



// filtered 3D gradient simple noise (few texture lookups, high quality)
// @param v >0
// @return random number in the range -1 .. 1
float SimplexNoise3D_TEX(float3 EvalPos)
{
	float3 OrthogonalPos = SkewSimplex(EvalPos);

	float3 PosA, PosB, PosC, PosD;
	float4 Weights = ComputeSimplexWeights3D(OrthogonalPos, PosA, PosB, PosC, PosD);

	// can be optimized to 1 or 2 texture lookups (4 or 8 channel encoded in 32 bit)
	float3 A = GetPerlinNoiseGradientTextureAt(PosA);
	float3 B = GetPerlinNoiseGradientTextureAt(PosB);
	float3 C = GetPerlinNoiseGradientTextureAt(PosC);
	float3 D = GetPerlinNoiseGradientTextureAt(PosD);
	
	PosA = UnSkewSimplex(PosA);
	PosB = UnSkewSimplex(PosB);
	PosC = UnSkewSimplex(PosC);
	PosD = UnSkewSimplex(PosD);

	float DistanceWeight;

	DistanceWeight = saturate(0.6f - length2(EvalPos - PosA));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float a = dot(A, EvalPos - PosA) * DistanceWeight;
	DistanceWeight = saturate(0.6f - length2(EvalPos - PosB));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float b = dot(B, EvalPos - PosB) * DistanceWeight;
	DistanceWeight = saturate(0.6f - length2(EvalPos - PosC));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float c = dot(C, EvalPos - PosC) * DistanceWeight;
	DistanceWeight = saturate(0.6f - length2(EvalPos - PosD));	DistanceWeight *= DistanceWeight; DistanceWeight *= DistanceWeight;
	float d = dot(D, EvalPos - PosD) * DistanceWeight;

	return 32 * (a + b + c + d);
}


float VolumeRaymarch(float3 posPixelWS, float3 posCameraWS)
{
	float ret = 0;
	int cnt = 60;

	LOOP for(int i=0; i < cnt; ++i)
	{
		ret += saturate(FastGradientPerlinNoise3D_TEX(lerp(posPixelWS, posCameraWS, i/(float)cnt) * 0.01) - 0.2f);
	}

	return ret / cnt * (length(posPixelWS - posCameraWS) * 0.001f );
}


