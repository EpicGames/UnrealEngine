// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// high frequency dither pattern appearing almost random without banding steps
//note: from "NEXT GENERATION POST PROCESSING IN CALL OF DUTY: ADVANCED WARFARE"
//      http://advances.realtimerendering.com/s2014/index.html
// Epic extended by FrameId
// ~7 ALU operations (2 frac, 3 mad, 2 *)
// @return 0..1
float InterleavedGradientNoise( float2 uv, float FrameId )
{
	// magic values are found by experimentation
	uv += FrameId * (float2(47, 17) * 0.695f);

    const float3 magic = float3( 0.06711056f, 0.00583715f, 52.9829189f );
    return frac(magic.z * frac(dot(uv, magic.xy)));
}
