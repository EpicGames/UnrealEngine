// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// 3D random number generator inspired by PCGs (permuted congruential generator)
// Using a **simple** Feistel cipher in place of the usual xor shift permutation step
// @param v = 3D integer coordinate
// @return three elements w/ 16 random bits each (0-0xffff).
// ~8 ALU operations for result.x    (7 mad, 1 >>)
// ~10 ALU operations for result.xy  (8 mad, 2 >>)
// ~12 ALU operations for result.xyz (9 mad, 3 >>)
uint3 Rand3DPCG16(int3 p)
{
	// taking a signed int then reinterpreting as unsigned gives good behavior for negatives
	uint3 v = uint3(p);

	// Linear congruential step. These LCG constants are from Numerical Recipies
	// For additional #'s, PCG would do multiple LCG steps and scramble each on output
	// So v here is the RNG state
	v = v * 1664525u + 1013904223u;

	// PCG uses xorshift for the final shuffle, but it is expensive (and cheap
	// versions of xorshift have visible artifacts). Instead, use simple MAD Feistel steps
	//
	// Feistel ciphers divide the state into separate parts (usually by bits)
	// then apply a series of permutation steps one part at a time. The permutations
	// use a reversible operation (usually ^) to part being updated with the result of
	// a permutation function on the other parts and the key.
	//
	// In this case, I'm using v.x, v.y and v.z as the parts, using + instead of ^ for
	// the combination function, and just multiplying the other two parts (no key) for 
	// the permutation function.
	//
	// That gives a simple mad per round.
	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;

	// only top 16 bits are well shuffled
	return v >> 16u;
}

// 3D random number generator inspired by PCGs (permuted congruential generator)
// Using a **simple** Feistel cipher in place of the usual xor shift permutation step
// http://jcgt.org/published/0009/03/02/
// @param v = 3D integer coordinate
// @return three elements w/ 32 random bits each (0-0xffffffff).
uint3 Rand3DPCG32(int3 p)
{
	// taking a signed int then reinterpreting as unsigned gives good behavior for negatives
	uint3 v = uint3(p);

	// Linear congruential step.
	v = v * 1664525u + 1013904223u;

	// shuffle
	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;

	// xoring high bits into low bits makes all 32 bits pretty good
	v ^= v >> 16u;

	// final shuffle
	v.x += v.y*v.z;
	v.y += v.z*v.x;
	v.z += v.x*v.y;

	return v;
}

// 4D random number generator inspired by PCGs (permuted congruential generator)
// Using a **simple** Feistel cipher in place of the usual xor shift permutation step
// http://jcgt.org/published/0009/03/02/
// @param v = 4D integer coordinate
// @return four elements w/ 32 random bits each (0-0xffffffff).
uint4 Rand4DPCG32(int4 p)
{
	// taking a signed int then reinterpreting as unsigned gives good behavior for negatives
	uint4 v = uint4(p);

	// Linear congruential step.
	v = v * 1664525u + 1013904223u;

	// shuffle
	v.x += v.y*v.w;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.w += v.y*v.z;

	// xoring high bits into low makes all 32 bits pretty good
	v ^= (v >> 16u);

	// final shuffle
	v.x += v.y*v.w;
	v.y += v.z*v.x;
	v.z += v.x*v.y;
	v.w += v.y*v.z;

	return v;
}

// Take uint value in [0,2^16) to float in [0,1)
float Rand16ToFloat(uint Rand16Bits)
{
	return float(Rand16Bits) * (1.0 / 65536.0);
}
float2 Rand16ToFloat(uint2 Rand16Bits)
{
	return float2(Rand16Bits) * (1.0 / 65536.0);
}
float3 Rand16ToFloat(uint3 Rand16Bits)
{
	return float3(Rand16Bits) * (1.0 / 65536.0);
}
float4 Rand16ToFloat(uint4 Rand16Bits)
{
	return float4(Rand16Bits) * (1.0 / 65536.0);
}

// Take 32-bit value to a float in [0,1) (uses the top most bits for compatibility with low-discrepancy constructions)
float Rand32ToFloat(uint Rand32Bits)
{
	return float(Rand32Bits >> 8) * 5.96046447754e-08;
}
float2 Rand32ToFloat(uint2 Rand32Bits)
{
	return float2(Rand32Bits >> 8) * 5.96046447754e-08;
}
float3 Rand32ToFloat(uint3 Rand32Bits)
{
	return float3(Rand32Bits >> 8) * 5.96046447754e-08;
}
float4 Rand32ToFloat(uint4 Rand32Bits)
{
	return float4(Rand32Bits >> 8) * 5.96046447754e-08;
}