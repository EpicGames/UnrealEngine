// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"

Texture2D<float4> AmbientOcclusionTexture;
SamplerState AmbientOcclusionTextureSampler;

void CompositeAmbientOcclusionPS(
	in noperspective float2 UV : TEXCOORD0,
	out float4 OutColor : SV_Target0
)
{
	float AmbientOcclusionMask = AmbientOcclusionTexture.Sample(AmbientOcclusionTextureSampler, UV).r;

	AmbientOcclusionMask = saturate(AmbientOcclusionMask);

	OutColor = float4(AmbientOcclusionMask, AmbientOcclusionMask, AmbientOcclusionMask, 1.0);
}
