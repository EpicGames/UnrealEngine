// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../SceneTextureParameters.ush"

#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 1
#define SUBSTRATE_FASTPATH 0
#define SUBSTRATE_SINGLEPATH 0
#define SUBSTRATE_COMPLEXSPECIALPATH 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0		// SUBSTRATE_TODO: tile the pass to work with glint
#define SUBSTRATE_GLINTS_ALLOWED 0
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateEvaluation.ush"
#endif

Texture2D<float4> SkyLightTexture;
SamplerState SkyLightTextureSampler;

void CompositeSkyLightPS(
	in noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0
)
{
	const float2 UV = OutUVAndScreenPos.xy;
	const uint2 PixelCoord = UV * View.BufferSizeAndInvSize.xy;

#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1


	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);

	const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);

	const float3 NullV = float3(0, 0, 1);
	const float3 NullL = float3(0, 0, 1);
	float3 Albedo = 0.0f;
	Substrate_for(uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
	{
		FSubstrateBSDF CurrentBSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
		FSubstrateBSDFContext Context = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, SubstrateAddressing, NullV);

		if (SubstrateIsBSDFVisible(CurrentBSDF))
		{
			FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;	// Fake unused in SubstrateCreateBSDFContext when using Forward inline shading
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, NullV, NullL);
			FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);

			// Use LuminanceWeightV instead of LuminanceWeight(..) as we only need to weight these value with the view transmittance, not the light transmittance;
			const float3 Weight = CurrentBSDF.LuminanceWeightV;

			Albedo += Weight * SubstrateGetBSDFDiffuseColor(CurrentBSDF);
		}
	}

#else

	FGBufferData GBufferData = GetGBufferDataFromSceneTextures(UV);
	float3 Albedo = GBufferData.StoredBaseColor - GBufferData.StoredBaseColor * GBufferData.Metallic;

#endif


	float4 SkyLight = SkyLightTexture.Sample(SkyLightTextureSampler, UV);
	// Apply albedo after denoising
	SkyLight.rgb *= Albedo;
	OutColor = SkyLight;
}
