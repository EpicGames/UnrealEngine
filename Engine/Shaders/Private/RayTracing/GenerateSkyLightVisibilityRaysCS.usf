// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	GenerateSkyLightVisibilityRaysCS.usf: Build variance map
===============================================================================================*/

#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "RayTracingCommon.ush"
#include "RayTracingSkyLightCommon.ush"
#include "SkyLightVisibilityRaysData.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"

uint3 SkyLightVisibilityRaysDimensions;
RWStructuredBuffer<FSkyLightVisibilityRays> OutSkyLightVisibilityRays;

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint2 PixelCoord = uint2(DispatchThreadId.x, DispatchThreadId.y);

	// split samples between strategies unless the skylight pdf is 0 due to MIS compensation (which implies a constant map)
	const float SkyLightSamplingStrategyPdf = SkyLight_Estimate() > 0 ? 0.5 : 0.0;

	for (uint SampleIndex = 0; SampleIndex < SkyLight.SamplesPerPixel; ++SampleIndex)
	{
		FRandomSequence RandSequence = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), SampleIndex, SkyLight.SamplesPerPixel);

		// Determine sky light or lambert ray
		float2 RandSample = RandSequence.Get2D();

		float3 RayDirection;
		float UniformPdf = 1.0 / (4 * PI);
		float SkyLightPdf;
		BRANCH
			if (RandSample.x < SkyLightSamplingStrategyPdf)
			{
				RandSample.x /= SkyLightSamplingStrategyPdf;

				FSkyLightSample SkySample = SkyLight_SampleLight(RandSample);

				RayDirection = SkySample.Direction;
				SkyLightPdf = SkySample.Pdf;
			}
			else
			{
				RandSample.x = (RandSample.x - SkyLightSamplingStrategyPdf) / (1.0 - SkyLightSamplingStrategyPdf);

				RayDirection = UniformSampleSphere(RandSample).xyz;
				SkyLightPdf = SkyLight_EvalLight(RayDirection).w;

				
			}

		float MisWeightOverPdf = 1.0 / lerp(UniformPdf, SkyLightPdf, SkyLightSamplingStrategyPdf);

		// Write out a new Sky Light Visibility Ray for this sample

		const uint SkyLightVisibilityRayIndex = GetSkyLightVisibilityRayIndex(PixelCoord, SampleIndex, SkyLightVisibilityRaysDimensions.xy);

		OutSkyLightVisibilityRays[SkyLightVisibilityRayIndex].DirectionAndPdf = float4(RayDirection, MisWeightOverPdf);
	}
}
