// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	MipTreeCommon.ush: Common utilities for MipTree-based sampling
===============================================================================================*/

#include "../MortonCode.ush"

#define USE_MORTON_CODE 1

uint2 DimensionsAtLevel(uint2 Dimensions, uint Level)
{
	return uint2(Dimensions.x >> Level, Dimensions.y >> Level);
}

uint BufferOffsetAtPixel(uint2 Pixel, uint Level, uint2 Dimensions)
{
	uint Offset = 0;
	for (uint LevelIndex = 0; LevelIndex < Level; ++LevelIndex)
	{
		uint2 MipLevelDimensions = DimensionsAtLevel(Dimensions, LevelIndex);
		Offset += MipLevelDimensions.x * MipLevelDimensions.y;
	}

	uint2 MipLevelDimensions = DimensionsAtLevel(Dimensions, Level);
#if USE_MORTON_CODE
	// NOTE: Morton encoding requires power-of-two dimensionality..
	Offset += MortonEncode(Pixel);
#else
	Offset += Pixel.y * MipLevelDimensions.x + Pixel.x;
#endif
	return Offset;
}
