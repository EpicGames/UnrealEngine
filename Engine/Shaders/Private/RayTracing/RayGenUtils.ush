// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define MAX_DISPATCH_THREAD_PER_DIMENSION 4096

int3 GetRayTracingThreadCountWrapped(uint TargetThreadCount, uint ThreadGroupSize)
{
	int3 ThreadCount = int3(TargetThreadCount, 1, 1);
	if (ThreadCount.x > MAX_DISPATCH_THREAD_PER_DIMENSION)
	{
		ThreadCount.y = (ThreadCount.x + ThreadGroupSize - 1) / ThreadGroupSize;
		ThreadCount.x = ThreadGroupSize;
	}

	if (ThreadCount.y > MAX_DISPATCH_THREAD_PER_DIMENSION)
	{
		ThreadCount.z = (ThreadCount.y + ThreadGroupSize - 1) / ThreadGroupSize;
		ThreadCount.y = ThreadGroupSize;
	}

	return ThreadCount;
}

uint GetUnWrappedRayTracingDispatchThreadId(int3 DispatchThreadId, uint ThreadGroupSize)
{
	return DispatchThreadId.x + (DispatchThreadId.z * ThreadGroupSize + DispatchThreadId.y) * ThreadGroupSize;
}
