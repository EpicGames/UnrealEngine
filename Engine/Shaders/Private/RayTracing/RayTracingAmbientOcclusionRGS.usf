// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Montecarlo.ush"
#include "../DeferredShadingCommon.ush"
#include "../SceneTextureParameters.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "../SphericalGaussian.ush"
#include "../Substrate/Substrate.ush"
#include "RayTracingCommon.ush"
#include "RayTracingDeferredShadingCommon.ush"


#define CONFIG_SHOOT_WITH_GEOMETRIC_NORMAL 1


uint SamplesPerPixel;
float MaxRayDistance;
float Intensity;
float MaxNormalBias;
RaytracingAccelerationStructure TLAS;
RWTexture2D<float> RWAmbientOcclusionMaskUAV;
RWTexture2D<float> RWAmbientOcclusionHitDistanceUAV;


/** Returns the radius of a pixel in world space. */
float ComputeWorldBluringRadiusCausedByPixelSize(float WorldDepth)
{
	// Should be multiplied 0.5* for the diameter to radius, and by 2.0 because GetTanHalfFieldOfView() cover only half of the pixels.
	return GetDepthPixelRadiusForProjectionType(WorldDepth);
}

void GenerateCosineNormalRay(
	uint2 PixelCoord,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float3 GeometricNormal,
	inout FRandomSequence RandSequence,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPDF
)
{
	// Draw random variable
	float2 BufferSize = View.BufferSizeAndInvSize.xy;
	float2 RandSample = RandSequence.Get2D();

	// Perform cosine-hemispherical sampling and convert to world-space
	float4 Direction_Tangent;
	#if 1
	{
		Direction_Tangent = CosineSampleHemisphereConcentric(RandSample);
		Direction_Tangent.w = Direction_Tangent.z;
	}
	#else
	{
		Direction_Tangent = CosineSampleHemisphere(RandSample);
	}
	#endif

	#if CONFIG_SHOOT_WITH_GEOMETRIC_NORMAL
		float3 Direction_World = TangentToWorld(Direction_Tangent.xyz, GeometricNormal);
	#else
		float3 Direction_World = TangentToWorld(Direction_Tangent.xyz, WorldNormal);
	#endif

	RayOrigin = TranslatedWorldPosition;
	RayDirection = Direction_World;
	RayTMin = 0.0;
	RayTMax = MaxRayDistance;
	RayPDF = Direction_Tangent.w;
}

float3 SampleNeightborPosition(uint2 PixelCoord, int2 Offset)
{
	uint2 ClampedPixelCoord = clamp(uint2(PixelCoord + Offset), uint2(View.ViewRectMin.xy), uint2(View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy - 1));

	float DeviceZ = SceneDepthTexture.Load(int3(ClampedPixelCoord, 0)).r;

	return ReconstructTranslatedWorldPositionFromDeviceZ(PixelCoord + Offset, DeviceZ);
}

RAY_TRACING_ENTRY_RAYGEN(AmbientOcclusionRGS)
{
	const uint2 PixelCoord = DispatchRaysIndex().xy + View.ViewRectMin.xy;

	#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));
	const bool bIsValid = IsSubstrateMaterial(TopLayerData);
	const bool bReconstructGeometryNormal = true; // SUBSTRATE_TODO: legacy avoids this on foliage, how do we do that?
	const float3 WorldNormal = TopLayerData.WorldNormal;
	#else
	const FGBufferData GBufferData = GetGBufferDataFromSceneTexturesLoad(PixelCoord);
	const bool bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
	const bool bReconstructGeometryNormal = GBufferData.ShadingModelID != SHADINGMODELID_TWOSIDED_FOLIAGE;
	const float3 WorldNormal = GBufferData.WorldNormal;
	#endif

	const float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	const float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	float3 TranslatedWorldPosition;
	float3 CameraDirection;
	ReconstructTranslatedWorldPositionAndCameraDirectionFromDeviceZ(PixelCoord, DeviceZ, TranslatedWorldPosition, CameraDirection);

	float3 GeometricNormal;
	float ShadingDotGeometric;
	if (bReconstructGeometryNormal && CONFIG_SHOOT_WITH_GEOMETRIC_NORMAL)
	{
		float WorldPixelRadius = ComputeWorldBluringRadiusCausedByPixelSize(SceneDepth);

		float3 E = SampleNeightborPosition(PixelCoord, int2( 1,  0)) - TranslatedWorldPosition;
		float3 W = SampleNeightborPosition(PixelCoord, int2(-1,  0)) - TranslatedWorldPosition;

		float3 N = SampleNeightborPosition(PixelCoord, int2( 0, -1)) - TranslatedWorldPosition;
		float3 S = SampleNeightborPosition(PixelCoord, int2( 0,  1)) - TranslatedWorldPosition;

		float DH = dot(E, View.ViewForward) + dot(W, View.ViewForward);
		float DV = dot(N, View.ViewForward) + dot(S, View.ViewForward);

		float3 H = (length2(E) < length2(W)) ? E : W;
		float3 V = (length2(S) < length2(N)) ? S : N;

		GeometricNormal = normalize(cross(H, V));

		FLATTEN
		if (dot(GeometricNormal, CameraDirection) > 0)
		{
			GeometricNormal = -GeometricNormal;
		}

		FSphericalGaussian CosineSG = ClampedCosine_ToSphericalGaussian(float3(0, 0, 1));
		FSphericalGaussian ShadingSG = ClampedCosine_ToSphericalGaussian(WorldNormal);
		FSphericalGaussian GeometricSG = ClampedCosine_ToSphericalGaussian(GeometricNormal);

		ShadingDotGeometric = saturate(Dot(ShadingSG, GeometricSG) * rcp(Dot(CosineSG, CosineSG)));

		// Detect the grass to fail grassfully and avoid 1pixel outlining.
		if (any(float2(DH, DV) < -5 * WorldPixelRadius))
		{
			GeometricNormal = WorldNormal;
			ShadingDotGeometric = 1.0;
		}
	}
	else
	{
		GeometricNormal = WorldNormal;
		ShadingDotGeometric = 1.0;
	}


	float RayCount = 0.0;
	float Visibility = 0.0;
	float ClosestRayHitDistance = 10000.0;
	// Declaring intensity as a local variable is a workaround for a shader compiler bug with NV drivers 430.39 and 430.64.
	const float IntensityLocal = Intensity;
	uint SamplesPerPixelLocal = SamplesPerPixel;
	
	// Mask out depth values beyond far plane
	bool IsFiniteDepth = DeviceZ > 0.0;
	bool bTraceRay = (IsFiniteDepth && bIsValid);
	if (!bTraceRay)
	{
		Visibility = 1.0;
		RayCount = SamplesPerPixel;
		SamplesPerPixelLocal = 0.0;
	}

	uint TimeSeed = View.StateFrameIndex;

	for (uint SampleIndex = 0; SampleIndex < SamplesPerPixelLocal; ++SampleIndex)
	{
		FRayDesc Ray;
		float RayPDF;

		FRandomSequence RandSequence = RandomSequenceCreate(uint3(PixelCoord, TimeSeed), SampleIndex, SamplesPerPixelLocal);

		GenerateCosineNormalRay(PixelCoord, TranslatedWorldPosition, WorldNormal, GeometricNormal, RandSequence, Ray.Origin, Ray.Direction, Ray.TMin, Ray.TMax, RayPDF);
		ApplyCameraRelativeDepthBias(Ray, PixelCoord, DeviceZ, WorldNormal, MaxNormalBias);

		//if (dot(WorldNormal, Ray.Direction) <= 0.0) // TODO(Denoiser): does it needs to be handled by the denoiser?
		//	continue;

		uint RayFlags = 0;
		const uint InstanceInclusionMask = RAY_TRACING_MASK_SHADOW | RAY_TRACING_MASK_THIN_SHADOW;

#if !ENABLE_MATERIALS
		RayFlags |= RAY_FLAG_FORCE_OPAQUE;
#endif
#if !ENABLE_TWO_SIDED_GEOMETRY
		RayFlags |= RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
#endif

		#if CONFIG_SHOOT_WITH_GEOMETRIC_NORMAL
			float RayWeight = max(dot(WorldNormal, Ray.Direction), 0.05) / max(RayPDF, 0.05);
		#else
			float RayWeight = 1.0;
		#endif

		FMinimalPayload MinimalPayload = TraceVisibilityRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			Ray);

		RayCount += RayWeight;
		Visibility += RayWeight * (MinimalPayload.IsMiss() ? 1.0 : (1.0 - IntensityLocal));
		if (MinimalPayload.IsHit())
		{
			ClosestRayHitDistance = min(ClosestRayHitDistance, MinimalPayload.HitT);
		}
	}

	// Output.
	{
		float2 RawOutput = 1;

		if (SamplesPerPixelLocal == 0)
		{
			RawOutput.x = 1;
			RawOutput.y = -2;
		}
		else if (RayCount == 0)
		{
			RawOutput.x = 1;
			RawOutput.y = -1;
		}
		else
		{
			RawOutput.x = ShadingDotGeometric * (Visibility / RayCount);
			RawOutput.y = ClosestRayHitDistance;
		}

		RWAmbientOcclusionMaskUAV[PixelCoord] = RawOutput.x;
		RWAmbientOcclusionHitDistanceUAV[PixelCoord] = RawOutput.y;
	}
}