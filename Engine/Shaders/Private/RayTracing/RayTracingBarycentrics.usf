// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Visualization.ush"
#include "RayTracingCommon.ush"

#if COMPUTESHADER
#include "RayTracingDeferredMaterials.ush"
#include "TraceRayInline.ush"
#endif // COMPUTESHADER

RWTexture2D<float4> Output;
RaytracingAccelerationStructure TLAS;
float RTDebugVisualizationNaniteCutError;

float TimingScale;

#if COMPUTESHADER
struct FNaniteTraceRayInlineCallback : FDefaultTraceRayInlineCallback
{
	float2 Barycentrics;
	float NaniteCutError;

	bool OnProceduralPrimitive(float3 ObjectRayOrigin, float3 ObjectRayDirection, float RayTMin, inout float RayTCurrent, FTraceRayInlineProceduralIntersectionParameters ProceduralIntersectionParameters)
	{
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
		uint PrimitiveID = ProceduralIntersectionParameters.InstanceID;
		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveID);
		uint PrimitiveInstanceIndex = 0; // TODO
		uint InstanceId = PrimitiveData.InstanceSceneDataOffset + PrimitiveInstanceIndex;
		FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceId);

		// TODO: implement support for intersection with procedural primitives

#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
		return false;
	}
};
#endif // COMPUTESHADER


void RayTracingBarycentricsCommon(uint2 DispatchThreadIndex)
{
	uint2 PixelCoord = DispatchThreadIndex.xy + View.ViewRectMin.xy;

	float2 RenderTargetUV = (float2(PixelCoord) + .5f) * View.BufferSizeAndInvSize.zw;

	FRayDesc Ray = CreatePrimaryRay(RenderTargetUV);
	uint RayFlags = 0
		| RAY_FLAG_CULL_BACK_FACING_TRIANGLES      // use back face culling
		| RAY_FLAG_FORCE_OPAQUE;                   // don't run anyhit shader

#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
	FTimestamp TimeBegin = GetShaderTimestamp();
#endif

	FDefaultPayload Payload = (FDefaultPayload)0;
#if COMPUTESHADER
	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();
	TraceRayInlineContext.bProcedural = ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE;

	FNaniteTraceRayInlineCallback NaniteCallback = (FNaniteTraceRayInlineCallback)0;
	NaniteCallback.NaniteCutError = RTDebugVisualizationNaniteCutError;

	FTraceRayInlineResult TraceRayResult = TraceRayInlineWithCallback(
		TLAS,   // AccelerationStructure
		RayFlags,
		RAY_TRACING_MASK_ALL,             // InstanceInclusionMask
		Ray.GetNativeDesc(),    // RayDesc
		TraceRayInlineContext,
		NaniteCallback
	);

	Payload.HitT = TraceRayResult.HitT;
	Payload.Barycentrics = TraceRayResult.bIsProcedural ? NaniteCallback.Barycentrics : TraceRayResult.Barycentrics;
#else
	TraceRay(
		TLAS,   // AccelerationStructure
		RayFlags,
		RAY_TRACING_MASK_ALL,             // InstanceInclusionMask
		RAY_TRACING_SHADER_SLOT_MATERIAL, // RayContributionToHitGroupIndex
		RAY_TRACING_NUM_SHADER_SLOTS,     // MultiplierForGeometryContributionToShaderIndex
		0,      // MissShaderIndex
		Ray.GetNativeDesc(),    // RayDesc
		Payload // Payload
	);
#endif // COMPUTESHADER

#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
	FTimestamp TimeEnd = GetShaderTimestamp();
#endif

	float4 Result = float4(0, 0, 0, 1);
	if (Payload.IsHit())
	{
		Result = float4(Payload.Barycentrics, 1.0f - Payload.Barycentrics.x - Payload.Barycentrics.y, 0.0f);
#if OUTPUT_TIMING && PLATFORM_SUPPORTS_SHADER_TIMESTAMP
		Result = float4(ColorMapInferno(float(ShaderTimestampDiff(TimeBegin, TimeEnd)) * TimingScale), 0.0f);
#endif
	}
	Output[PixelCoord] = Result;
}

#if COMPUTESHADER

[numthreads(INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X, INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y, 1)]
void RayTracingBarycentricsMainCS(uint2 DispatchThreadIndex : SV_DispatchThreadID)
{
	RayTracingBarycentricsCommon(DispatchThreadIndex);
}

#else

RAY_TRACING_ENTRY_RAYGEN(RayTracingBarycentricsMainRGS)
{
	RayTracingBarycentricsCommon(DispatchRaysIndex().xy);
}

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingBarycentricsMainCHS,
	FDefaultPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.Barycentrics = Attributes.GetBarycentrics();
	Payload.HitT = RayTCurrent();
}

#endif