// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingTypes.h"
#include "/Engine/Shared/PathTracingDefinitions.h"

StructuredBuffer<FRayTracingDecal> SceneDecals;
uint SceneDecalCount;

float3 SceneDecalsTranslatedBoundMin;
float3 SceneDecalsTranslatedBoundMax;
uint DecalGridResolution;
uint DecalGridMaxCount;
uint DecalGridAxis;

RWTexture2DArray<uint> RWDecalGrid;
RWBuffer<uint> RWDecalGridData;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void BuildDecalGridCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId >= DecalGridResolution) || DispatchThreadId.z >= 3)
	{
		return;
	}
	// figure out dimension of the 3d grid and the current ID to be filled
	uint3 VoxelId = 0, VoxelRes = 1;
	int Axis = DispatchThreadId.z;
	if (DecalGridAxis != Axis && DecalGridAxis < 3)
	{
		RWDecalGrid[DispatchThreadId] = 0;
		return;
	}
	switch (Axis)
	{
		case 0: VoxelId.yz = DispatchThreadId.xy; VoxelRes.yz = DecalGridResolution; break;
		case 1: VoxelId.xz = DispatchThreadId.xy; VoxelRes.xz = DecalGridResolution; break;
		case 2: VoxelId.xy = DispatchThreadId.xy; VoxelRes.xy = DecalGridResolution; break;
	}
	// get bounding box of current voxel
	float3 Lo = lerp(SceneDecalsTranslatedBoundMin, SceneDecalsTranslatedBoundMax, float3(VoxelId + 0) / float3(VoxelRes));
	float3 Hi = lerp(SceneDecalsTranslatedBoundMin, SceneDecalsTranslatedBoundMax, float3(VoxelId + 1) / float3(VoxelRes));

	uint NumVisible = 0;
	uint MaxVisible = min(DecalGridMaxCount, RAY_TRACING_DECAL_COUNT_MAXIMUM);
	uint Offset = DecalGridMaxCount * (Axis + 3 * (DispatchThreadId.x + DecalGridResolution * DispatchThreadId.y));

	for (uint DecalIndex = 0; DecalIndex < SceneDecalCount; DecalIndex++)
	{
		FRayTracingDecal Decal = SceneDecals[DecalIndex];

		if (any(Hi < Decal.TranslatedBoundMin) || any(Lo > Decal.TranslatedBoundMax))
		{
			continue;
		}

#if 1
		// Transform the corners of the voxel into decal space so we can cull the "other way" as well and account for rotated decals
		float3 DecalPos000 = mul(float4(Lo.x, Lo.y, Lo.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos001 = mul(float4(Lo.x, Lo.y, Hi.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos010 = mul(float4(Lo.x, Hi.y, Lo.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos011 = mul(float4(Lo.x, Hi.y, Hi.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos100 = mul(float4(Hi.x, Lo.y, Lo.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos101 = mul(float4(Hi.x, Lo.y, Hi.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos110 = mul(float4(Hi.x, Hi.y, Lo.z, 1), Decal.TranslatedWorldToDecal).xyz;
		float3 DecalPos111 = mul(float4(Hi.x, Hi.y, Hi.z, 1), Decal.TranslatedWorldToDecal).xyz;

		float3 DecalLo = min(min(min(DecalPos000, DecalPos001), min(DecalPos010, DecalPos011)), min(min(DecalPos100, DecalPos101), min(DecalPos110, DecalPos111)));
		float3 DecalHi = max(max(max(DecalPos000, DecalPos001), max(DecalPos010, DecalPos011)), max(max(DecalPos100, DecalPos101), max(DecalPos110, DecalPos111)));

		if (any(DecalHi < -1) || any(DecalLo > 1))
		{
			continue;
		}
#endif

		RWDecalGridData[Offset + NumVisible] = Decal.CallableSlotIndex;
		NumVisible++;
		// TODO: handle overflow better?
		if (NumVisible >= MaxVisible)
		{
			break;
		}
	}

	RWDecalGrid[DispatchThreadId] = NumVisible;
}
