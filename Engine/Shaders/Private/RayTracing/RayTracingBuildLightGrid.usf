// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "/Engine/Shared/LightDefinitions.h"
#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingTypes.h"
#include "../IntersectionUtils.ush"

uint SceneLightCount;
StructuredBuffer<FRTLightingData> SceneLights;
uint SceneInfiniteLightCount;
float3 SceneLightsTranslatedBoundMin;
float3 SceneLightsTranslatedBoundMax;
RWTexture2DArray<uint> RWLightGrid;
RWBuffer<uint> RWLightGridData;
uint LightGridResolution;
uint LightGridMaxCount;
uint LightGridAxis;


#define RECT_TEST 2 // 0: half-space only
                    // 1: barndoors (planes only)
					// 2: barndoors+aabb culling

#define CONE_TEST 1 // 0: test against cone AABB
					// 1: accurate cone/aabb test


[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
void RayTracingBuildLightGridCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId >= LightGridResolution) || DispatchThreadId.z >= 3)
	{
		return;
	}
	// figure out dimension of the 3d grid and the current ID to be filled
	uint3 VoxelId = 0, VoxelRes = 1;
	int Axis = DispatchThreadId.z;
	if (Axis != LightGridAxis && LightGridAxis < 3)
	{
		RWLightGrid[DispatchThreadId] = 0;
		return;
	}
	switch (Axis)
	{
		case 0: VoxelId.yz = DispatchThreadId.xy; VoxelRes.yz = LightGridResolution; break;
		case 1: VoxelId.xz = DispatchThreadId.xy; VoxelRes.xz = LightGridResolution; break;
		case 2: VoxelId.xy = DispatchThreadId.xy; VoxelRes.xy = LightGridResolution; break;
	}
	// get bounding box of current voxel
	float3 Lo = lerp(SceneLightsTranslatedBoundMin, SceneLightsTranslatedBoundMax, float3(VoxelId + 0) / float3(VoxelRes));
	float3 Hi = lerp(SceneLightsTranslatedBoundMin, SceneLightsTranslatedBoundMax, float3(VoxelId + 1) / float3(VoxelRes));

	uint NumVisible = 0;
	uint MaxVisible = LightGridMaxCount;
	uint Offset = LightGridMaxCount * (Axis + 3 * (DispatchThreadId.x + LightGridResolution * DispatchThreadId.y));
	for (uint LightIndex = SceneInfiniteLightCount; LightIndex < SceneLightCount; LightIndex++)
	{
		uint LightType = SceneLights[LightIndex].Type;
		float3 Center = SceneLights[LightIndex].TranslatedLightPosition;
		float Radius = 1.0 / SceneLights[LightIndex].InvRadius;
		float3 Normal = -SceneLights[LightIndex].Direction;

		if (!AABBOverlapsSphere(Lo, Hi, Center, Radius))
		{
			// bounds don't overlap there sphere of influence
			continue;
		}

		// reject against normal
		if (LightType != LIGHT_TYPE_POINT)
		{
			if (IsBoxBehindPlane(Lo, Hi, Normal, Center))
			{
				continue;
			}
		}
#if RECT_TEST > 0
		if (LightType == LIGHT_TYPE_RECT)
		{
			// check barndoors
			const float BarnCos = SceneLights[LightIndex].RectLightBarnCosAngle;
			if (BarnCos > 0.035)
			{
				const float BarnLen = SceneLights[LightIndex].RectLightBarnLength;
				const float HalfWidth = SceneLights[LightIndex].SourceRadius;
				const float HalfHeight = SceneLights[LightIndex].SourceLength;


				if (!AABBOverlapsRectLightFrustum(Lo, Hi, Center, Normal, SceneLights[LightIndex].Tangent, HalfWidth, HalfHeight, Radius, BarnCos, BarnLen, RECT_TEST == 2))
				{
					continue;
				}
			}
		}
#endif
		if (LightType == LIGHT_TYPE_SPOT)
		{
			const float CosOuter = SceneLights[LightIndex].SpotAngles.x;
			if (!AABBOverlapsCurvedCone(Lo, Hi, Center, Normal, CosOuter, Radius, CONE_TEST == 0))
			{
				continue;
			}
		}

		{
			RWLightGridData[Offset + NumVisible] = LightIndex;
			NumVisible++;
			// TODO: handle overflow better?
			if (NumVisible >= MaxVisible)
			{
				break;
			}
		}
	}
	RWLightGrid[DispatchThreadId] = NumVisible;
}
