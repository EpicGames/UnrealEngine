// Copyright Epic Games, Inc. All Rights Reserved.


#include "../Common.ush"
#include "RayTracingCommon.ush"

RaytracingAccelerationStructure               TLAS;
StructuredBuffer<FBasicRayData>               Rays;
RWStructuredBuffer<uint>                      OcclusionOutput;
RWStructuredBuffer<FIntersectionPayload>      IntersectionOutput;

RAY_TRACING_ENTRY_RAYGEN(OcclusionMainRGS)
{
	const uint RayIndex = DispatchRaysIndex().x;
	FBasicRayData InputRay = Rays[RayIndex];

	RayDesc Ray;
	Ray.Origin = InputRay.Origin;
	Ray.Direction = InputRay.Direction;
	Ray.TMin = 0.0f;
	Ray.TMax = InputRay.TFar;

	uint RayFlags = 0
	              | RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH // terminate immediately upon detecting primitive intersection
	              | RAY_FLAG_FORCE_OPAQUE                    // don't run anyhit shader
	              | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER;        // don't run closesthit shader
	const uint InstanceInclusionMask = RAY_TRACING_MASK_OPAQUE;

	FDefaultPayload Payload = (FDefaultPayload)0;

	TraceRay(
		TLAS,   // AccelerationStructure
		RayFlags,
		InstanceInclusionMask,
		RAY_TRACING_SHADER_SLOT_MATERIAL, // RayContributionToHitGroupIndex
		RAY_TRACING_NUM_SHADER_SLOTS,     // MultiplierForGeometryContributionToShaderIndex
		0,      // MissShaderIndex
		Ray,    // RayDesc
		Payload // Payload
	);

	// Workaround for DXC bug creating validation error
	//OcclusionOutput[RayIndex] = Payload.IsHit() ? ~0 : 0;
	OcclusionOutput[RayIndex] = ((FMinimalPayload)Payload).IsHit() ? ~0 : 0;
}

RAY_TRACING_ENTRY_RAYGEN(IntersectionMainRGS)
{
	const uint RayIndex = DispatchRaysIndex().x;
	FBasicRayData InputRay = Rays[RayIndex];

	RayDesc Ray;
	Ray.Origin = InputRay.Origin;
	Ray.Direction = InputRay.Direction;
	Ray.TMin = 0.0f;
	Ray.TMax = InputRay.TFar;

	uint RayFlags = RAY_FLAG_FORCE_OPAQUE; // don't run anyhit shader

	FDefaultPayload Payload = (FDefaultPayload)0;

	TraceRay(
		TLAS,   // AccelerationStructure
		RayFlags,
		RAY_TRACING_MASK_OPAQUE,          // InstanceInclusionMask
		RAY_TRACING_SHADER_SLOT_MATERIAL, // RayContributionToHitGroupIndex
		RAY_TRACING_NUM_SHADER_SLOTS,     // MultiplierForGeometryContributionToShaderIndex
		0,      // MissShaderIndex
		Ray,    // RayDesc
		Payload // Payload
	);

	IntersectionOutput[RayIndex] = Payload;
}

RAY_TRACING_ENTRY_CLOSEST_HIT(IntersectionMainCHS,
	FDefaultPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.HitT = RayTCurrent();
	Payload.Barycentrics = Attributes.GetBarycentrics();
	Payload.InstanceIndex = InstanceIndex();
	Payload.PrimitiveIndex = PrimitiveIndex();
}

RAY_TRACING_ENTRY_CLOSEST_HIT(DefaultMainCHS,
	FDefaultPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.Barycentrics = Attributes.GetBarycentrics();
	Payload.InstanceID = InstanceID();
	Payload.InstanceIndex = InstanceIndex();
	Payload.PrimitiveIndex = PrimitiveIndex();
	Payload.HitT = RayTCurrent();
}

RAY_TRACING_ENTRY_ANY_HIT(DefaultOpaqueAHS,
	FDefaultPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
}

RAY_TRACING_ENTRY_MISS(DefaultPayloadMS, FDefaultPayload, Payload)
{
	Payload.SetMiss();
}

RAY_TRACING_ENTRY_MISS(PackedMaterialClosestHitPayloadMS, FPackedMaterialClosestHitPayload, Payload)
{
	Payload.SetMiss();
}
