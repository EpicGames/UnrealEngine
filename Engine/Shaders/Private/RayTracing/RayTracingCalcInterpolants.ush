// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Barycentrics.ush"

struct FRaytracingDerivatives
{
	TDual< float3 > WorldPosition;
	TDual< float3 > WorldGeoNormals;
	TDual< float4 > ScreenPosition;
	
#if NUM_TEX_COORD_INTERPOLATORS
	TDual< float2 > TexCoords[NUM_TEX_COORD_INTERPOLATORS];
#endif
};

void CalcInterpolants(
	in float4 SvPosition, 
	in FRayCone RayCone, 
	in FRayTracingIntersectionAttributes Attributes, 
	inout FVertexFactoryInterpolantsVSToPS OutInterpolants, 
	inout FRaytracingDerivatives OutDerivateData, 
	inout float3 OutFaceNormal)
{
	FVertexFactoryRayTracingInterpolants Interpolated = (FVertexFactoryRayTracingInterpolants)0;

	float4 ClipPositions[3];
	float3 WorldPositions[3];
	float3 WorldGeoNormals[3];
	float2 TexCoords[3];
#if NUM_TEX_COORD_INTERPOLATORS
	float2 TexCoords_Custom[NUM_TEX_COORD_INTERPOLATORS][3];
#endif

	const float2 VW = Attributes.GetBarycentrics();
	const float3 UVW = float3(1 - VW.x - VW.y, VW.x, VW.y);

	for (int i = 0; i < 3; i++)
	{
		FVertexFactoryInput Input = LoadVertexFactoryInputForHGS(PrimitiveIndex(), i);

		FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
		float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
		float4 WorldPositionExcludingWPO = VertexFactoryGetWorldPosition(Input, VFIntermediates);
		FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPositionExcludingWPO.xyz, TangentToLocal);

		FVertexFactoryRayTracingInterpolants PerVertexInterpolants = VertexFactoryGetRayTracingInterpolants(Input, VFIntermediates, VertexParameters);
		Interpolated = VertexFactoryInterpolate(PerVertexInterpolants, UVW[i], Interpolated, 1.0);

		WorldPositions[i] = WorldPositionExcludingWPO.xyz;
		WorldGeoNormals[i] = VertexFactoryGetWorldNormal(Input, VFIntermediates);
		TexCoords[i] = VertexFactoryGetRayTracingTextureCoordinate(PerVertexInterpolants);

		#if (NUM_TEX_COORD_INTERPOLATORS)
		UNROLL
		for (uint TexCoordIndex=0;TexCoordIndex<NUM_TEX_COORD_INTERPOLATORS; ++TexCoordIndex)
		{
		#if !VERTEX_FACTORY_MODIFIES_INTERPOLATION && !VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
			TexCoords_Custom[TexCoordIndex][i] = GetUV(PerVertexInterpolants.InterpolantsVSToPS, TexCoordIndex);
		#else
			TexCoords_Custom[TexCoordIndex][i] = TexCoords[i];
		#endif
		}
		#endif

		ClipPositions[i] = mul(float4(WorldPositions[i], 1), View.TranslatedWorldToClip);
	}

	OutInterpolants = VertexFactoryAssignInterpolants(Interpolated);

	float3 PA = WorldPositions[1] - WorldPositions[0];
	float3 PB = WorldPositions[2] - WorldPositions[0];
	float3 Unnormalized = cross(PB, PA);
	float InvWorldArea = rsqrt(dot(Unnormalized, Unnormalized));
	OutFaceNormal = Unnormalized * InvWorldArea;

#if USE_RAYTRACED_TEXTURE_RAYCONE_LOD && (NUM_TEX_COORD_INTERPOLATORS || USE_PARTICLE_SUBUVS) && !VERTEX_FACTORY_MODIFIES_INTERPOLATION
	float2 TA = TexCoords[1] - TexCoords[0];
	float2 TB = TexCoords[2] - TexCoords[0];
	float UvArea = abs((TA.x * TB.y) - (TA.y * TB.x));

	float RayConeCapArea = RayCone.Width * RayCone.Width * (1.0f / (2.0f * PI));
	float3 RayDir = WorldRayDirection();
	float RaySlope = dot(RayDir, OutFaceNormal);
	const float SmallValue = 1.0 / 16777216.0;
	GlobalRayCone_TexArea = (UvArea * InvWorldArea * RayConeCapArea) / max(RaySlope * RaySlope, SmallValue);
	GlobalRayCone_TexArea = max(GlobalRayCone_TexArea, SmallValue);
#endif

	// Compute derivatives
	const float2 PixelClip = (SvPosition.xy - View.ViewRectMin.xy) * View.ViewSizeAndInvSize.zw * float2(2, -2) + float2(-1, 1);
	FBarycentrics Barycentrics = CalculateTriangleBarycentrics(PixelClip, ClipPositions[0], ClipPositions[1], ClipPositions[2], View.ViewSizeAndInvSize.zw);
	OutDerivateData.WorldPosition = Lerp( WorldPositions[0], WorldPositions[1], WorldPositions[2], Barycentrics );
	OutDerivateData.WorldGeoNormals = Lerp( WorldGeoNormals[0], WorldGeoNormals[1], WorldGeoNormals[2], Barycentrics );
	OutDerivateData.ScreenPosition = Lerp( ClipPositions[0] * rcp(ClipPositions[0].w), ClipPositions[1] * rcp(ClipPositions[1].w), ClipPositions[2] * rcp(ClipPositions[2].w), Barycentrics );

#if NUM_TEX_COORD_INTERPOLATORS > 0
	UNROLL
	for (uint TexCoordIndex = 0; TexCoordIndex < NUM_TEX_COORD_INTERPOLATORS; TexCoordIndex++)
	{
		OutDerivateData.TexCoords[TexCoordIndex] = Lerp( TexCoords_Custom[TexCoordIndex][0], TexCoords_Custom[TexCoordIndex][1], TexCoords_Custom[TexCoordIndex][2], Barycentrics );
	}
#endif
}

void CalcInterpolants(in FRayCone RayCone, in FRayTracingIntersectionAttributes Attributes, inout FVertexFactoryInterpolantsVSToPS OutInterpolants, inout float3 OutFaceNormal)
{
	FRaytracingDerivatives OutDerivatives; // unused
	CalcInterpolants(0, RayCone, Attributes, OutInterpolants, OutDerivatives, OutFaceNormal);
}

// simplified call when Geometric Normal is not needed
void CalcInterpolants(in FRayCone RayCone, in FRayTracingIntersectionAttributes Attributes, inout FVertexFactoryInterpolantsVSToPS OutInterpolants)
{
	float3 OutGeoNormal = 0; // unused
	FRaytracingDerivatives OutDerivatives; // unused
	CalcInterpolants(0, RayCone, Attributes, OutInterpolants, OutDerivatives, OutGeoNormal);
}

void ApplyRaytracingDerivatives(inout FMaterialPixelParameters Out, in FRaytracingDerivatives In)
{
	Out.WorldPosition_DDX = In.WorldPosition.Value_dx;
	Out.WorldPosition_DDY = In.WorldPosition.Value_dy;

	Out.ScreenPosition_DDX = In.ScreenPosition.Value_dx;
	Out.ScreenPosition_DDY = In.ScreenPosition.Value_dy;

	Out.WorldGeoNormal_DDX = In.WorldGeoNormals.Value_dx;
	Out.WorldGeoNormal_DDY = In.WorldGeoNormals.Value_dy;

#if NUM_TEX_COORD_INTERPOLATORS
	UNROLL
	for (uint TexCoordIndex = 0; TexCoordIndex < NUM_TEX_COORD_INTERPOLATORS; TexCoordIndex++)
	{
		Out.TexCoords_DDX[TexCoordIndex] = In.TexCoords[TexCoordIndex].Value_dx;
		Out.TexCoords_DDY[TexCoordIndex] = In.TexCoords[TexCoordIndex].Value_dy;
	}
#endif
}