// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../LightData.ush"
#include "../CapsuleLightSampling.ush"

bool GenerateCapsuleLightOcclusionRayWithSolidAngleSampling(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPdf
)
{
	float3 LightDirection = LightParameters.TranslatedWorldPosition - TranslatedWorldPosition;
	float RayLength2 = dot(LightDirection, LightDirection);
	float Radius = max(LightParameters.SourceRadius, 0.01); // clamp radius to a small value since the code below doesn't handle pure lines
	float Radius2 = Radius * Radius;
	float SourceLength = LightParameters.SourceLength;
	float3 Axis = LightParameters.Tangent;

	// compute a bounding representation for the capsule
	FCapsuleSphericalBounds CapsuleBounds = CapsuleGetSphericalBounds(LightDirection, Axis, Radius, SourceLength);
	// sample a direction toward these bounds
	float4 Result = SampleCapsuleBounds(CapsuleBounds, RandSample);
	// compute distance to capsule (and check that we hit it)
	float Distance = CapsuleTest(Result.xyz, LightDirection, Axis, Radius2, SourceLength);
	// sample is only valid if we hit the light
	if (Distance > 0)
	{
		RayOrigin = TranslatedWorldPosition;
		RayDirection = Result.xyz;
		RayTMin = 0;
		RayTMax = Distance;
		RayPdf = Result.w;
		return true;
	}
	else
	{
		RayOrigin = 0;
		RayDirection = 0;
		RayTMin = 0;
		RayTMax = 0;
		RayPdf = 0;
		return false;
	}
}
