// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RayTracingCommon.ush: common header used in multiple ray generation shaders
=============================================================================*/

#pragma once

#ifndef RAYTRACINGCOMMON_USH_INCLUDED
#define RAYTRACINGCOMMON_USH_INCLUDED // Workarround for UE-66460

// Make sure we recompile ray tracing shaders if the main shader version changes
#include "/Engine/Public/ShaderVersion.ush"

// Update this GUID to invalidate RayTracing shaders and force a shader recompilation.
#pragma message("UESHADERMETADATA_VERSION 82EF3461-600F-45F4-94A7-F2D9EF032386")

#include "../ShadingCommon.ush"
#include "../OctahedralCommon.ush"
#include "../SceneData.ush"
#include "../Common.ush"

#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingPayloadType.h"
#include "RayTracingRayCone.ush"

#if SUBSTRATE_ENABLED && MATERIAL_IS_SUBSTRATE
#define SUBSTRATE_INLINE_SHADING 1
#endif

#include "../Substrate/Substrate.ush"

#ifdef OVERRIDE_RAYTRACING_USH
#include "/Platform/Private/RayTracing.ush"
#endif // OVERRIDE_RAYTRACING_USH

#if COMPILER_METAL
#include "/Engine/Private/RayTracing/RayTracingCommonMetal.ush"
#endif // COMPILER_METAL

// Certain DXR system value intrinsics are explicitly disallowed in Unreal Engine entirely.
// Supported cross-platform system value intrinsics:
//                                  RayGen   Intersection   AnyHit  ClosestHit  Miss  Callable
// uint3 DispatchRaysIndex()          YES         -           -          -        -       -
// uint3 DispatchRaysDimensions()     YES         -           -          -        -       -
// float3 WorldRayOrigin()             -         YES*        YES        YES      YES      -
// float3 WorldRayDirection()          -         YES*        YES        YES      YES      -
// float RayTMin()                     -          -           -          -        -       -
// float RayTCurrent()                 -         YES*        YES        YES      YES      -
// float RayFlags()                    -          -           -          -        -       -
// uint InstanceIndex()                -         YES*        YES        YES       -       -
// uint InstanceID()                   -         YES*        YES        YES       -       -
// uint GeometryIndex()                -          -           -          -        -       -
// uint PrimitiveIndex()               -         YES*        YES        YES       -       -
// float3 ObjectRayOrigin()            -          -           -          -        -       -
// float3 ObjectRayDirection()         -          -           -          -        -       -
// float3x4 ObjectToWorld3x4()         -          -           -          -        -       -
// float4x3 ObjectToWorld4x3()         -          -           -          -        -       -
// float3x4 WorldToObject3x4()         -          -           -          -        -       -
// float4x3 WorldToObject4x3()         -          -           -          -        -       -
// uint HitKind()                      -          -          YES        YES       -       -
// * NOTE: Intersection shaders are only supported in DXR.

#define ALLOW_ALL_DXR_INTRINSICS 1

#if !ALLOW_ALL_DXR_INTRINSICS
#if !RAYGENSHADER
// These intrinsics are allowed by DXR in all shader types, but we only support them in RayGen.
// If values are required, they must be explicitly passed through the payload structure.
#define DispatchRaysIndex()      DispatchRaysIndex_is_only_supported_in_raygen_shaders
#define DispatchRaysDimensions() DispatchRaysDimensions_is_only_supported_in_raygen_shaders
#endif // !RAYGENSHADER

// These DXR intrinsics are disallowed for better cross-platform compatibility and performance.
#define RayTMin()                RayTMin_is_not_supported
#define RayFlags()               RayFlags_is_not_supported
#define GeometryIndex()          GeometryIndex_is_not_supported
#define ObjectRayOrigin()        ObjectRayOrigin_is_not_supported
#define ObjectRayDirection()     ObjectRayDirection_is_not_supported
#define ObjectToWorld3x4()       ObjectToWorld3x4_is_not_supported
#define ObjectToWorld4x3()       ObjectToWorld4x3_is_not_supported
#define WorldToObject3x4()       WorldToObject3x4_is_not_supported
#define WorldToObject4x3()       WorldToObject4x3_is_not_supported
#endif // !ALLOW_ALL_DXR_INTRINSICS

// Define generic wrappers for ray tracing shader entry points, if not already overriden

#ifndef RAY_TRACING_ENTRY_RAYGEN
#define RAY_TRACING_ENTRY_RAYGEN(name)\
[shader("raygeneration")] void name()
#endif // RAY_TRACING_ENTRY_RAYGEN

#ifndef RAY_TRACING_ENTRY_INTERSECTION
#define RAY_TRACING_ENTRY_INTERSECTION(name)\
[shader("intersection")] void name()
#endif //RAY_TRACING_ENTRY_INTERSECTION

#ifndef RAY_TRACING_ENTRY_CLOSEST_HIT
#define RAY_TRACING_ENTRY_CLOSEST_HIT(name, payload_type, payload_name, attributes_type, attributes_name)\
[shader("closesthit")] void name(inout payload_type payload_name, in attributes_type attributes_name)
#endif //RAY_TRACING_ENTRY_CLOSEST_HIT

#ifndef RAY_TRACING_ENTRY_ANY_HIT
#define RAY_TRACING_ENTRY_ANY_HIT(name, payload_type, payload_name, attributes_type, attributes_name)\
[shader("anyhit")] void name(inout payload_type payload_name, in attributes_type attributes_name)
#endif // RAY_TRACING_ENTRY_ANY_HIT

#ifndef RAY_TRACING_ENTRY_MISS
#define RAY_TRACING_ENTRY_MISS(name, payload_type, payload_name)\
[shader("miss")] void name(inout payload_type payload_name)
#endif //RAY_TRACING_ENTRY_MISS

#ifndef RAY_TRACING_ENTRY_CALLABLE
#define RAY_TRACING_ENTRY_CALLABLE(name, params_type, params_name)\
[shader("callable")] void name(inout params_type params_name)
#endif //RAY_TRACING_ENTRY_CALLABLE

// Helper macro to set common entry point for passes that have both inline (CS) and non-inline (RGS) version.
// Given "name" it will create "nameCS" entry point for compute and "nameRGS" for raygen shader.
#ifndef RAY_TRACING_ENTRY_RAYGEN_OR_INLINE

#if COMPUTESHADER && PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#define RAY_TRACING_ENTRY_RAYGEN_OR_INLINE(name)\
void name##_INTERNAL(uint3 DispatchThreadIndex);\
[numthreads(INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X, INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y, 1)]\
void name##CS(uint3 DispatchThreadIndex : SV_DispatchThreadID) {\
	name##_INTERNAL(DispatchThreadIndex);}\
void name##_INTERNAL(uint3 DispatchThreadIndex)

#else // COMPUTESHADER && PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#define RAY_TRACING_ENTRY_RAYGEN_OR_INLINE(name)\
void name##_INTERNAL(uint3 DispatchThreadIndex);\
RAY_TRACING_ENTRY_RAYGEN(name##RGS){\
name##_INTERNAL(DispatchRaysIndex());}\
void name##_INTERNAL(uint3 DispatchThreadIndex)

#endif // COMPUTESHADER && PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#endif // RAY_TRACING_ENTRY_RAYGEN_OR_INLINE

#ifndef OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR

// Must match D3D12_RAYTRACING_INSTANCE_FLAGS / VkGeometryInstanceFlagBitsKHR
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_NONE								0
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE				0x1
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_FRONT_COUNTERCLOCKWISE	0x2
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE						0x4
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE					0x8

#endif // OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR

// must match ERayTracingInstanceFlags
#define RAYTRACING_INSTANCE_FLAG_NONE					0
#define RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE	(1 << 1)
#define RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_REVERSE	(1 << 2)
#define RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE			(1 << 3)
#define RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE		(1 << 4)

#define RAY_TRACING_BLEND_MODE_OPAQUE				0
#define RAY_TRACING_BLEND_MODE_ALPHA_COMPOSITE		1
#define RAY_TRACING_BLEND_MODE_TRANSLUCENT			2
#define RAY_TRACING_BLEND_MODE_ADDITIVE				3
#define RAY_TRACING_BLEND_MODE_MODULATE				4
#define RAY_TRACING_BLEND_MODE_ALPHA_HOLDOUT		5

// Check how Flags are packed in the FPackedMaterialClosestHitPayload before adding to ensure bits are free (current packing gives us 6 bits for input and output)
// There are two sets of flags:
//   - "inputs" are set before calling TraceRay
//   - "outputs" are set during execution of the CHS and are valid after TraceRay returns
// These two sets of masks can overlap because they never co-exist
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_NONE                               0
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE                   (1 << 0) // Indicates that ray has hit front face of a primitive. Set by closest hit shader.
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED                    (1 << 1) // Ray hit a two-sided material
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT                   (1 << 2) // We hit a primitive flagged as a holdout
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_HAS_INDIRECT_LIGHTING        (1 << 3) // Payload contains valid indirect lighting
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_UNUSED4                      (1 << 4) // Free for future use
#define RAY_TRACING_PAYLOAD_OUTPUT_FLAG_UNUSED5                      (1 << 5) // Free for future use

#define RAY_TRACING_PAYLOAD_INPUT_FLAG_NONE								   0	
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_MINIMAL_PAYLOAD               (1 << 0) // Indicates that closest hit shader should only fill FMinimalPayload (HitT), skipping full material evaluation. Set by RayGen shader, before TraceRay().
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_ENABLE_SKY_LIGHT_CONTRIBUTION (1 << 1) // Indicates that hit shaders should add in the Sky Light contribution in their output. Set by RayGen shader, before TraceRay().
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_LUMEN_PAYLOAD		         (1 << 2) // Indicates that closest hit shader should evaluate geometry normal and instanceid.
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT            (1 << 3) // Indicates that translucent materials should be ignored by IgnoreHit() in anyhit shaders.
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY                    (1 << 4) // Indicates this ray is being traced as part of a raytraced shadow calculation (see TraceVisibilityRay).
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_CAMERA_RAY                    (1 << 5) // Indicates this ray is being traced directly from the camera
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_INDIRECT_LIGHTING_RAY		 (1 << 6) // Indicates this ray is ray is a part of indirect lighting (GI or reflections) calculations
#define RAY_TRACING_PAYLOAD_INPUT_FLAG_UNUSED						 (1 << 7) // Free for future use

///////////////////////////////////////////////////////////////////////////////////////////////////
// Instance descriptor

#ifndef OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR
#define OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR

// Must match D3D12_RAYTRACING_INSTANCE_DESC / VkAccelerationStructureInstanceKHR
struct FPlatformRayTracingInstanceDescriptor
{
	float3x4 Transform;
	uint InstanceIdAndMask;
	uint InstanceContributionToHitGroupIndexAndFlags;
	uint2 AccelerationStructure; //uint64

	uint GetInstanceId()
	{
		return InstanceIdAndMask & 0x00FFFFFF;
	}

	uint GetMask()
	{
		return (InstanceIdAndMask >> 24) & 0xFF;
	}

	uint GetInstanceContributionToHitGroupIndex()
	{
		return InstanceContributionToHitGroupIndexAndFlags & 0x00FFFFFF;
	}

	uint GetFlags()
	{
		return (InstanceContributionToHitGroupIndexAndFlags >> 24) & 0xFF;
	}
};

void SetRayTracingInstanceTransform(inout FPlatformRayTracingInstanceDescriptor Desc, float3x4 LocalToWorld)
{
	Desc.Transform = LocalToWorld;
}

uint GetRayTracingInstanceDescriptorInstanceId(FPlatformRayTracingInstanceDescriptor Desc)
{
	return Desc.GetInstanceId();
}

uint GetRayTracingInstanceDescriptorMask(FPlatformRayTracingInstanceDescriptor Desc)
{
	return Desc.GetMask();
}

uint GetRayTracingInstanceDescriptorInstanceContributionToHitGroupIndex(FPlatformRayTracingInstanceDescriptor Desc)
{
	return Desc.GetInstanceContributionToHitGroupIndex();
}

uint GetRayTracingInstanceDescriptorFlags(FPlatformRayTracingInstanceDescriptor Desc)
{
	return Desc.GetFlags();
}

FPlatformRayTracingInstanceDescriptor BuildPlatformRayTracingInstanceDesc(uint InstanceMask, uint InstanceId, uint InstanceFlags, uint InstanceContributionToHitGroupIndex, float3x4 LocalToWorld, uint2 AccelerationStructureAddress)
{
	FPlatformRayTracingInstanceDescriptor Desc;
	Desc.InstanceIdAndMask = (InstanceMask << 24) | (InstanceId & 0x00FFFFFF);
	Desc.InstanceContributionToHitGroupIndexAndFlags = (InstanceFlags << 24) | (InstanceContributionToHitGroupIndex & 0x00FFFFFF);
	Desc.AccelerationStructure = AccelerationStructureAddress;
	SetRayTracingInstanceTransform(Desc, LocalToWorld);

	return Desc;
}

#endif // !OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR

///////////////////////////////////////////////////////////////////////////////////////////////////
// Attributes and small payload

// Indices and vertex positions of the ray tracing geometry triangle.
// Used for loading and interpolation of per-vertex attributes and computing per-triangle normals.
struct FTriangleBaseAttributes
{
	uint3 Indices;
	float3 LocalPositions[3];
};

struct FBasicRayData
{
	float3 Origin;
	uint Mask;
	float3 Direction;
	float TFar;
};

struct FMinimalPayload
{
	float HitT; // Distance from ray origin to the intersection point in the ray direction. Negative on miss.

	bool IsMiss() { return HitT < 0; }
	bool IsHit() { return !IsMiss(); }

	void SetMiss() { HitT = -1; }
};
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_MINIMAL)
CHECK_RT_PAYLOAD_SIZE(FMinimalPayload)
#endif

struct FIntersectionPayload : FMinimalPayload
{
	uint   PrimitiveIndex; // Index of the primitive within the geometry inside the bottom-level acceleration structure instance. Undefined on miss.
	uint   InstanceIndex;  // Index of the current instance in the top-level structure. Undefined on miss.
	float2 Barycentrics;   // Primitive barycentric coordinates of the intersection point. Undefined on miss.
};

struct FDefaultPayload : FIntersectionPayload
{
	uint   InstanceID; // Value of FRayTracingGeometryInstance::UserData. Undefined on miss.
};
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_DEFAULT)
CHECK_RT_PAYLOAD_SIZE(FDefaultPayload)
#endif


#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_VFX)
struct FVFXTracePayload : FMinimalPayload
{
	uint   GPUSceneInstanceId;
	float2 Barycentrics;
	float3 TranslatedWorldPosition;
	float3 WorldNormal;
};
CHECK_RT_PAYLOAD_SIZE(FVFXTracePayload)
#endif

#define RAY_TRACING_DEFERRED_MATERIAL_KEY_INVALID   0xFFFFFFFF
#define RAY_TRACING_DEFERRED_MATERIAL_KEY_RAY_MISS  0xFFFFFFFE

struct FDeferredMaterialPayload : FMinimalPayload
{
	uint   SortKey;          // MaterialID by default
	uint   PixelCoordinates; // X in low 16 bits, Y in high 16 bits
};
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_DEFERRED)
CHECK_RT_PAYLOAD_SIZE(FDeferredMaterialPayload)
#endif


#ifndef RAY_TRACING_SUPPORT_CUSTOM_UINT_IN_INTERSECTION_ATTRIBUTES
#	define RAY_TRACING_SUPPORT_CUSTOM_UINT_IN_INTERSECTION_ATTRIBUTES 0
#endif

struct FRayTracingIntersectionAttributes
{
	uint2 PackedData;

	float2 GetBarycentrics()
	{
#if RAY_TRACING_SUPPORT_CUSTOM_UINT_IN_INTERSECTION_ATTRIBUTES
		return UnpackUnorm2x16(PackedData.x);
#else
		return asfloat(PackedData);
#endif
	}

	void SetBarycentrics(float2 Barycentrics)
	{
#if RAY_TRACING_SUPPORT_CUSTOM_UINT_IN_INTERSECTION_ATTRIBUTES
		PackedData.x = PackUnorm2x16(Barycentrics);
#else
		PackedData = asuint(Barycentrics);
#endif
	}

#if RAY_TRACING_SUPPORT_CUSTOM_UINT_IN_INTERSECTION_ATTRIBUTES
	void SetCustomData(uint Data)
	{
		PackedData.y = Data;
	}
#endif

};

// Include utility fonction for world position reconstruction 
#include "../PositionReconstructionCommon.ush" 

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers

uint CalcLinearIndex(uint2 PixelCoord)
{
	return PixelCoord.y * uint(View.BufferSizeAndInvSize.x) + PixelCoord.x;
}

uint2 GetPixelCoord(uint2 DispatchThreadId, uint UpscaleFactor)
{
	uint UpscaleFactorPow2 = UpscaleFactor * UpscaleFactor;

	// TODO(Denoiser): find a way to not interfer with TAA's jittering.
	uint SubPixelId = View.StateFrameIndex & (UpscaleFactorPow2 - 1);

	return DispatchThreadId * UpscaleFactor + uint2(SubPixelId & (UpscaleFactor - 1), SubPixelId / UpscaleFactor);
}

// WorldNormal is the vector towards which the ray position will be offseted.
void ApplyPositionBias(inout float3 RayOrigin, float3 RayDirection, const float3 WorldNormal, const float MaxNormalBias)
{
	// Apply normal perturbation when defining ray to:
	// * avoid self intersection with current underlying triangle
	// * hide mismatch between shading surface & geometric surface
	//
	// While using shading normal is not correct (we should use the 
	// geometry normal, but it is not available atm/too costly to compute), 
	// it is good enough for a cheap solution
	const float MinBias = 0.01f;
	const float MaxBias = max(MinBias, MaxNormalBias);
	const float NormalBias = lerp(MaxBias, MinBias, saturate(dot(WorldNormal, RayDirection)));

	RayOrigin += WorldNormal * NormalBias;
}

void ApplyCameraRelativeDepthBias(inout float3 RayOrigin, float3 RayDirection, uint2 PixelCoord, float DeviceZ, const float3 WorldNormal, const float AbsoluteNormalBias)
{
	float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDeviceZ(PixelCoord, DeviceZ);
	float3 CameraDirection = TranslatedWorldPosition - View.TranslatedWorldCameraOrigin;
	float DistanceToCamera = GetDistanceToCameraFromViewVector(CameraDirection);
	float Epsilon = 1.0e-4;
	float RelativeBias = DistanceToCamera * Epsilon;
	//float ProjectedBias = RelativeBias / dot(RayDirection, WorldNormal);

	float RayBias = max(RelativeBias, AbsoluteNormalBias);
	RayOrigin -= GetCameraVectorFromTranslatedWorldPosition(CameraDirection) * RayBias;
	ApplyPositionBias(RayOrigin, RayDirection, WorldNormal, RayBias);
}

// This strategy is similar to ApplyPositionBias, but is the version used in the path tracer
void ApplyRayBias(inout float3 RayOrigin, float HitT, float3 GeoNormal)
{
	// Take maximum of position or hit distance to determine "radius" of hit avoidance
	const float RefValue = max(max(abs(RayOrigin.x), abs(RayOrigin.y)), max(abs(RayOrigin.z), HitT));
	const uint UlpRadius = 16; // number of floating point ulps to skip around an intersection
	const float RelativeOffset = asfloat(asuint(RefValue) + UlpRadius) - RefValue;
	const float BaseOffset = 0.00390625; // 2^-8cm (roughly 39micro meters) (This avoids precision issues very close to the origin where the ulps become too small)
	RayOrigin += max(BaseOffset, RelativeOffset) * GeoNormal;
}

float3 TranslatedWorldToTLASWorld(float3 TranslatedWorldPosition)
{
	const FDFVector3 TLASPreViewTranslation = MakeDFVector3(View.TLASPreViewTranslationHigh, View.TLASPreViewTranslationLow);
	const float3 TLASWorldPosition = TranslatedWorldPosition - DFFastSubtractDemote(PrimaryView.PreViewTranslation, TLASPreViewTranslation);
	return TLASWorldPosition;
}

float3 TLASWorldToTranslatedWorld(float3 TLASWorldPosition)
{
	const FDFVector3 TLASPreViewTranslation = MakeDFVector3(View.TLASPreViewTranslationHigh, View.TLASPreViewTranslationLow);
	const float3 TranslatedWorldPosition = TLASWorldPosition + DFFastSubtractDemote(PrimaryView.PreViewTranslation, TLASPreViewTranslation);
	return TranslatedWorldPosition;
}

float4 TranslatedWorldPositionToSvPosition(float3 TranslatedWorldPosition)
{
	const float4 ClipSpacePosition = mul(float4(TranslatedWorldPosition, 1.0f), ResolvedView.TranslatedWorldToClip);
	const float4 SvPosition = float4((ClipSpacePosition.xy / ClipSpacePosition.w * float2(.5f, - .5f) + .5f) * ResolvedView.ViewSizeAndInvSize.xy, ClipSpacePosition.zw);
	return SvPosition;
}

// Pack normal vector into an uint32 using Octahedron encoding
uint PackNormalToUInt32(float3 Normal)
{
	float2 NormalAsOctahedron = UnitVectorToOctahedron(Normal);
	uint2 QuantizedOctahedron = clamp((NormalAsOctahedron * 0.5 + 0.5) * 65535.0 + 0.5, 0.0, 65535.0);
	return ((QuantizedOctahedron.x & 0xFFFF) << 16) | (QuantizedOctahedron.y & 0xFFFF);
}

// Pack normal vector into from an uint32 using Octahedron encoding
float3 UnpackNormalFromUInt32(uint PackedNormal)
{
	uint2 QuantizedOctahedron = uint2((PackedNormal >> 16) & 0xFFFF, PackedNormal & 0xFFFF);
	float2 WorldNormalAsOctahedron = ((QuantizedOctahedron / 65535.0) - 0.5) * 2.0;
	float3 WorldNormal = OctahedronToUnitVector(WorldNormalAsOctahedron);
	return WorldNormal;
}

// Recommended default value for TMax for rays that should traverse the largest possible distance
#define RAY_DEFAULT_T_MAX (FLOAT_MAX)

#if !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

struct FRayDesc
{
	float3 Origin;
	float TMin;
	float3 Direction;
	float TMax;

	RayDesc GetNativeDesc()
	{
		RayDesc NativeRay;
		NativeRay.Origin = TranslatedWorldToTLASWorld(Origin);
		NativeRay.TMin = TMin;
		NativeRay.Direction = Direction;
		NativeRay.TMax = TMax;
		return NativeRay;
	}
};

float3 TranslatedWorldRayOrigin()
{
	return TLASWorldToTranslatedWorld(WorldRayOrigin());
}

void ApplyPositionBias(inout FRayDesc Ray, const float3 WorldNormal, const float MaxNormalBias)
{
	ApplyPositionBias(Ray.Origin, Ray.Direction, WorldNormal, MaxNormalBias);
}

void ApplyCameraRelativeDepthBias(inout FRayDesc Ray, uint2 PixelCoord, float DeviceZ, const float3 WorldNormal, const float AbsoluteNormalBias)
{
	ApplyCameraRelativeDepthBias(Ray.Origin, Ray.Direction, PixelCoord, DeviceZ, WorldNormal, AbsoluteNormalBias);
}

FRayDesc CreatePrimaryRay(float2 UV)
{
	float2 ScreenPosition = (UV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	FRayDesc Ray;
	float4 NearPoint = mul(float4(ScreenPosition, 1, 1), View.ClipToTranslatedWorld);
	float4 FarPoint = mul(float4(ScreenPosition, 0, 1), View.ClipToTranslatedWorld);
	Ray.Origin = NearPoint.xyz * rcp(NearPoint.w);
	Ray.Direction = normalize(NearPoint.w * FarPoint.xyz - FarPoint.w * NearPoint.xyz);

	// extract far plane (ortho only, so ignore distinction between T and Z)
	float4 Back = mul(float4(0, 0, 0, 1), View.ClipToView);
	Ray.TMin = 0;
	Ray.TMax = Back.w > 0 ? (Back.z / Back.w - View.NearPlane) : RAY_DEFAULT_T_MAX;

	return Ray;
}

#endif // !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

uint TranslateRayTracingInstanceFlags(uint InFlags)
{
	uint Result = PLATFORM_RAYTRACING_INSTANCE_FLAG_NONE;

	if (InFlags & RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE)
	{
		Result |= PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE;
	}

	if ((InFlags & RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_REVERSE) == 0)
	{
		// Counterclockwise is the default for UE. Reversing culling is achieved by *not* setting this flag.
		Result |= PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_FRONT_COUNTERCLOCKWISE;
	}

	if (InFlags & RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE)
	{
		Result |= PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE;
	}

	if (InFlags & RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE)
	{
		Result |= PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE;
	}

	return Result;
}

uint TranslateRayTracingPlatformInstanceFlags(uint InPlatformFlags)
{
	uint Result = RAYTRACING_INSTANCE_FLAG_NONE;

	if (InPlatformFlags & PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE)
	{
		Result |= RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE;
	}

	if ((InPlatformFlags & PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_FRONT_COUNTERCLOCKWISE) == 0)
	{
		Result |= RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_REVERSE;
	}

	if (InPlatformFlags & PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE)
	{
		Result |= RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE;
	}

	if (InPlatformFlags & PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE)
	{
		Result |= RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE;
	}

	return Result;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Material payload

#if !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#if SUBSTRATE_GBUFFER_FORMAT==1

struct FMaterialClosestHitPayload : FMinimalPayload
{
	// float FMinimalPay	// float FMinimalPayload::HitT
	FRayCone RayCone;
	float3 Radiance;
	float IorOverride; // If >= 0, the material overridden IOR value
	float3 WorldNormal; // Top Layer Normal
	float Opacity;
	float3 TransmittancePreCoverage;
	uint BlendingMode;
	uint PrimitiveLightingChannelMask;
	float3 IndirectIrradiance;
	uint ShadingModelID;
	float3 RayDirection;
	bool bIsThinSurface;

	// Quite some code assume FRayHitInfo has a WorldPos
	// So keep it here and force to re-construct it in Unpack call using ray information.
	// It is not packed in FRayHitInfoPacked
	float3 TranslatedWorldPos;			
	uint Flags;							
	FSubstrateRaytracingPayload SubstrateData; 

	void SetFrontFace()						{ Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE; }
	bool IsFrontFace()						{ return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }

	void SetTwoSided()						{ Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED; }
	bool IsTwoSided()						{ return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED) != 0; }

	void SetHoldout()                       { Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT; }
	bool IsHoldout()                        { return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT) != 0; }

	void SetHasIndirectLighting()           { Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_HAS_INDIRECT_LIGHTING; }
	bool HasIndirectLighting()				{ return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_HAS_INDIRECT_LIGHTING) != 0; }

	FRayCone GetRayCone()					{ return RayCone; }
	void SetRayCone(FRayCone In)			{ RayCone = In; }
	float3 GetRayDirection()				{ return RayDirection; }
};

struct FPackedMaterialClosestHitPayload : FMinimalPayload
{
	// Flags layout
	// 8bits                     | 8bits      | 3bits     | 3bits            | 2bits | 8bits
	// RAY_TRACING_PAYLOAD_INPUT | MIP Bias   | BlendMode | LightChannelMask | Free  | Opacity
	 
	// float FMinimalPayload::HitT                       // 4  bytes
	uint PackedRayCone;                                  // 4  bytes (fp16 width, fp16 spread) aliased with RayDirection for MS lighting payload
	uint Flags;											 // 4  bytes (see layout above)
	uint TransmittancePreCoverage;						 // 4  bytes
	uint Radiance;										 // 4  bytes
	uint PackedGeometryNormal;							 // 4  bytes
	uint SceneInstanceIndex;                             // 4  bytes - GPUSceneInstanceIndex
														 //=28 bytes - must be in sync with InternalGetRayTracingMaterialPayloadSizeInBytes()
	FSubstrateRaytracingPayload SubstrateData;			 //  ? bytes - must be in sync with GetRaytracingMaterialPayloadSize (see FSubstrateRaytracingPayload)
														 //=28+? bytes total

	void SetMinimalPayloadMode()			{ Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_MINIMAL_PAYLOAD; }
	bool IsMinimalPayloadMode()				{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_MINIMAL_PAYLOAD) != 0; }

	void SetEnableSkyLightContribution()	{ Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_ENABLE_SKY_LIGHT_CONTRIBUTION; }
	bool IsEnableSkyLightContribution()		{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_ENABLE_SKY_LIGHT_CONTRIBUTION) != 0; }

	void SetLumenPayload()					{ Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_LUMEN_PAYLOAD; }
	bool IsLumenPayload()					{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_LUMEN_PAYLOAD) != 0; }

	void SetIgnoreTranslucentMaterials()	{ Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT; }
	bool IsIgnoreTranslucentMaterials()		{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT) != 0; }

	void SetShadowRay()                     { Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY; }
	bool IsShadowRay()						{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY) != 0; }

	void SetCameraRay()                     { Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_CAMERA_RAY; }
	bool IsCameraRay()						{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_CAMERA_RAY) != 0; }

	void SetIndirectLightingRay()           { Flags |= RAY_TRACING_PAYLOAD_INPUT_FLAG_INDIRECT_LIGHTING_RAY; }
	bool IsIndirectLightingRay()			{ return (Flags & RAY_TRACING_PAYLOAD_INPUT_FLAG_INDIRECT_LIGHTING_RAY) != 0; }

	FRayCone GetRayCone()					{ return UnpackRayCone(PackedRayCone); }
	void SetRayCone(FRayCone NewRayCone)	{ PackedRayCone = PackRayCone(NewRayCone); }

	float GetMipBias()						{ return UnpackR8(Flags>>8u); }
	void SetMipBias(float NewMipBias)		{ Flags = (Flags & 0xFFFF00FFFu) | (PackR8(NewMipBias) << 8u); }

	void SetRadiance(in float3 In)			{ Radiance = PackR11G11B10F(In); }
	float3 GetRadiance()					{ return UnpackR11G11B10F(Radiance); }

	// For now we use the last material data slot to store the indirect irradiance in order to not increase the payload over 64bytes
	void SetIndirectIrradiance(float3 In) 	{ if ((Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_HAS_INDIRECT_LIGHTING) != 0) { SubstrateData.Data[SUBSTRATE_RT_PAYLOAD_NUM_UINTS - 1] = PackR11G11B10F(In); } }
	float3 GetIndirectIrradiance()			{ return ((Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_HAS_INDIRECT_LIGHTING) != 0) ? UnpackR11G11B10F(SubstrateData.Data[SUBSTRATE_RT_PAYLOAD_NUM_UINTS-1]) : 0.f; }

	float3 GetWorldNormal()					{ return SubstrateUnpackTopLayerData(SubstrateData.PackedTopLayerData).WorldNormal; }

	void SetGeometryNormal(float3 GeometryNormal) 
	{
		PackedGeometryNormal = PackNormalToUInt32(GeometryNormal); 
	}

	float3 GetGeometryNormal()
	{ 
		return UnpackNormalFromUInt32(PackedGeometryNormal); 
	}
	
	float3 GetShadowVisibility()
	{
		return UnpackRGB111110(TransmittancePreCoverage); // TODO: Does this interfere with the BRDF evaluation in the lighting miss shader?
	}

	void SetShadowVisibility(float3 ShadowVisibility)
	{
		TransmittancePreCoverage = PackRGB111110(ShadowVisibility);
	}

	uint GetLightIndex()
	{
		return SceneInstanceIndex;
	}

	void SetLightIndex(uint LightIndex)
	{
		SceneInstanceIndex = LightIndex;
	}

	bool IsValid()							{ return any(SubstrateData.PackedTopLayerData != 0); }
	uint  GetShadingModelID()				{ return IsValid() ? SHADINGMODELID_SUBSTRATE : SHADINGMODELID_UNLIT; }
	uint  GetFlags()						{ return (Flags & 0xFF); }
	bool IsTwoSided()						{ return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED) != 0; }
	bool IsFrontFace()						{ return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }
	uint  GetBlendingMode()					{ return (Flags >> 16u) & 0x7; }
	uint  GetPrimitiveLightingChannelMask() { return (Flags >> 19u) & 0x7; }
	bool  IsThinSurface() 					{ return (Flags >> 22u) & 0x1; }
	float GetOpacity()						{ return UnpackR8(Flags >> 24u); }
	float3 GetTransmittancePreCoverage()	{ return UnpackR11G11B10F(TransmittancePreCoverage); }

	void SetFlags(uint In)							{ Flags |= ((In & 0xFF)); }
	void SetBlendingMode(uint In)					{ Flags  = (Flags & 0xFFF8FFFF) | ((In & 0x7)<<16u); }
	void SetPrimitiveLightingChannelMask(uint In)	{ Flags  = (Flags & 0xFFC7FFFF) | ((In & 0x7)<<19u); }
	void SetIsThinSurface(bool In)					{ Flags  = (Flags & 0xFFBFFFFF) | ((In & 0x1)<<22u); }
	void SetOpacity(float In)						{ Flags  = (Flags & 0x00FFFFFF) | (PackR8(In) << 24u); }
	void SetTransmittancePreCoverage(float3 In)		{ TransmittancePreCoverage  = PackR11G11B10F(In); }

	void SetRayDirection(float3 In)			{ PackedRayCone = PackNormalToUInt32(In); }
	float3 GetRayDirection()				{ return UnpackNormalFromUInt32(PackedRayCone); }

	// Member function for (IsLumenPayload() == 1)
	void SetSceneInstanceIndex(uint In)		{ SceneInstanceIndex = In; }
	int GetSceneInstanceIndex()				{ return SceneInstanceIndex; }
	// SceneInstanceIndex is not used in hit group shaders so reuse it to pass a random number
	void SetRandom(float E)					{ SetSceneInstanceIndex(asuint(E)); }
	float GetRandom()						{ return asfloat(GetSceneInstanceIndex()); }

	// (Placeholder) Accessors used by the legacy refleciton API
#if SUBSTRATE_LEGACY_REFLECTION_API
	float3 GetWorldTangent()				{ return float3(1,0,0); }
	float3 GetBaseColor()					{ return 1.f; }
	float4 GetCustomData()					{ return 0.f; }
	float GetMetallic()						{ return 1.f; }
	float GetSpecular()						{ return 0.5f; }
	float GetRoughness()					{ return 0.5f; }
	float GetIor()							{ return 1.f; }
	float GetAnisotropy()					{ return 0.f; }
	float3 GetDiffuseColor()				{ return 1.f; }
	float3 GetSpecularColor()				{ return 1.f; }
#endif
};

FPackedMaterialClosestHitPayload PackRayTracingPayload(FMaterialClosestHitPayload In, in FRayCone RayCone)
{
	FPackedMaterialClosestHitPayload Out = (FPackedMaterialClosestHitPayload)0;
	Out.HitT = In.HitT;
	Out.SetRayCone(RayCone);
	Out.SetFlags(In.Flags);
	Out.SetBlendingMode(In.BlendingMode);
	Out.SetPrimitiveLightingChannelMask(In.PrimitiveLightingChannelMask);
	Out.SetRadiance(In.Radiance);
	Out.SubstrateData = In.SubstrateData;
	Out.SetIndirectIrradiance(In.IndirectIrradiance);
	Out.SetOpacity(In.Opacity);
	Out.SetTransmittancePreCoverage(In.TransmittancePreCoverage);
	Out.SetIsThinSurface(In.bIsThinSurface);
	return Out;
}

FMaterialClosestHitPayload UnpackRayTracingPayload(FPackedMaterialClosestHitPayload In, FRayDesc Ray)
{
	FMaterialClosestHitPayload Out = (FMaterialClosestHitPayload)0;
	Out.HitT = In.HitT;
	Out.RayCone = In.GetRayCone();
	Out.BlendingMode = In.GetBlendingMode();
	Out.PrimitiveLightingChannelMask = In.GetPrimitiveLightingChannelMask();
	Out.Flags = In.GetFlags();
	Out.Radiance = In.GetRadiance();
	Out.WorldNormal = In.GetWorldNormal();
	Out.ShadingModelID = In.GetShadingModelID();
	Out.IndirectIrradiance = In.GetIndirectIrradiance();
	Out.SubstrateData = In.SubstrateData;
	Out.TranslatedWorldPos = Ray.Origin + Out.HitT * Ray.Direction;
	Out.RayDirection = In.GetRayDirection();
	Out.Opacity = In.GetOpacity();
	Out.TransmittancePreCoverage = In.GetTransmittancePreCoverage();
	Out.bIsThinSurface = In.IsThinSurface();
	return Out;
}

#else // SUBSTRATE_GBUFFER_FORMAT==1

struct FMaterialClosestHitPayload : FMinimalPayload
{
										// Unpacked  Packed
										// offset    bytes
	// float FMinimalPayload::HitT		// 0         4       32bits
	FRayCone RayCone;					// 4         8       64bits
	float3 Radiance;					// 8         6       48bits
	float3 WorldNormal;					// 24        6       48bits
	float3 BaseColor;					// 36        6       48bits
	float3 DiffuseColor;				// 48        0       (derived)
	float3 SpecularColor;				// 60        0       (derived)
	float Opacity;						// 72        2       16bits
	float Metallic;						// 76        1       8bits
	float Specular;						// 80        1       8bits
	float Roughness;					// 84        2       16bits
	float Ior;							// 88        2       16bits
	uint ShadingModelID;				// 92        1       4bits
	uint BlendingMode;					// 96        0       3bits (packed with ShadingModelID)
	uint PrimitiveLightingChannelMask;	// 100       0       3bits (packed with ShadingModelID)
	float4 CustomData;					// 104       4       32bits
	float GBufferAO;					// 120       0       (removed)
	float3 IndirectIrradiance;			// 124       0       48bits -- gbuffer only has float payload and there are truncation HLSL warnings

	// Quite some code assume FRayHitInfo has a WorldPos
	// So keep it here and force to re-construct it in Unpack call using ray information.
	// It is not packed in FRayHitInfoPacked
	float3 TranslatedWorldPos;			// 136       0       (derived)
	uint Flags;							// 148       0       6bits (packed with ShadingModelID)
	float3 WorldTangent;				// 152       6       48bits
	float Anisotropy;					// 164       2       16bits (packed with WorldTangent)
										// 166 total

	void SetFrontFace() { Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE; }
	bool IsFrontFace() { return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }

	void SetTwoSided() { Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED; }
	bool IsTwoSided() { return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED) != 0; }

	void SetHoldout() { Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT; }
	bool IsHoldout() { return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_IS_HOLDOUT) != 0; }
	
	FRayCone GetRayCone() { return RayCone; }
	void SetRayCone(FRayCone NewRayCone) { RayCone = NewRayCone; }

	bool HasAnisotropy() { return abs(Anisotropy) >= 0.001f; }

	float3 GetRayDirection() { return IndirectIrradiance; /* Alias IndirectIrradiance/RayDirection */  } 
};

struct FPackedMaterialClosestHitPayload : FMinimalPayload
{
	// float FMinimalPayload::HitT                       // 4  bytes

	uint PackedRayCone;                                  // 4 bytes (fp16 width, fp16 spread)
	// 8 bits - MipBias
	// 8 bits - Flags
	// 16 bits - Unused
	uint FlagsAndMipBias;

	uint RadianceAndNormal[3];                           // 12 bytes
	uint BaseColorAndOpacity[2];                         // 8  bytes
	uint MetallicAndSpecularAndRoughness;                // 4  bytes
	// 16 bits - IOR
	// 4 bits - Shading Model Id
	// 3 bits - Blending Mode
	// 6 bits - Unused
	// 3 bits - Lighting Channel Mask
	uint IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask;    // 4  bytes
	uint PackedIndirectIrradiance[2];                    // 8  bytes
	uint PackedCustomData;                               // 4  bytes
	uint WorldTangentAndAnisotropy[2];                   // 8  bytes
	uint PackedGeometryNormal;                           // 4  bytes
	                                                     // 64 bytes total

	void SetMinimalPayloadMode() { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_MINIMAL_PAYLOAD; }
	bool IsMinimalPayloadMode() { return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_MINIMAL_PAYLOAD) != 0; }

	void SetEnableSkyLightContribution() { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_ENABLE_SKY_LIGHT_CONTRIBUTION ; }
	bool IsEnableSkyLightContribution() { return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_ENABLE_SKY_LIGHT_CONTRIBUTION) != 0; }

	void SetLumenPayload() { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_LUMEN_PAYLOAD; }
	bool IsLumenPayload() { return  (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_LUMEN_PAYLOAD) != 0;}

	void SetIgnoreTranslucentMaterials() { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT; }
	bool IsIgnoreTranslucentMaterials() { return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_IGNORE_TRANSLUCENT) != 0; }

	void SetShadowRay()                     { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY; }
	bool IsShadowRay()						{ return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY) != 0; }

	void SetCameraRay()                     { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_CAMERA_RAY; }
	bool IsCameraRay()						{ return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_CAMERA_RAY) != 0; }

	void SetIndirectLightingRay()           { FlagsAndMipBias |= RAY_TRACING_PAYLOAD_INPUT_FLAG_INDIRECT_LIGHTING_RAY; }
	bool IsIndirectLightingRay()			{ return (FlagsAndMipBias & RAY_TRACING_PAYLOAD_INPUT_FLAG_INDIRECT_LIGHTING_RAY) != 0; }

	FRayCone GetRayCone() { return UnpackRayCone(PackedRayCone); }
	void SetRayCone(FRayCone NewRayCone) { PackedRayCone = PackRayCone(NewRayCone); }

	float GetMipBias()						{ return UnpackR8(FlagsAndMipBias >> 8u); }
	void SetMipBias(float NewMipBias)		{ FlagsAndMipBias = (FlagsAndMipBias & 0xFFFF00FFu) | (PackR8(NewMipBias) << 8u); }

	void SetRadiance(in float3 InRadiance)
	{
		RadianceAndNormal[0]  = f32tof16(InRadiance.x) | (f32tof16(InRadiance.y) << 16);
		RadianceAndNormal[1] &= 0xffff << 16; // Clear the radiance (low bits), keep packed normal component (high bits)
		RadianceAndNormal[1] |= f32tof16(InRadiance.z);
	}

	float3 GetRadiance()
	{
		float3 Result;
		Result.x = f16tof32(RadianceAndNormal[0]);
		Result.y = f16tof32(RadianceAndNormal[0] >> 16);
		Result.z = f16tof32(RadianceAndNormal[1]);
		return Result;
	}

	void SetIndirectIrradiance(float3 InIndirectIrradiance)
	{
		PackedIndirectIrradiance[0]  = f32tof16(InIndirectIrradiance.x);
		PackedIndirectIrradiance[0] |= f32tof16(InIndirectIrradiance.y) << 16;
		PackedIndirectIrradiance[1]  = f32tof16(InIndirectIrradiance.z) | (PackedIndirectIrradiance[1] & 0xFFFF0000u);
	}

	float3 GetIndirectIrradiance()
	{
		float3 Result;
		Result.x = f16tof32(PackedIndirectIrradiance[0]);
		Result.y = f16tof32(PackedIndirectIrradiance[0] >> 16);
		Result.z = f16tof32(PackedIndirectIrradiance[1]);
		return Result;
	}
	
	// Alias IndirectIrradiance for storing ray direction
	void SetRayDirection(float3 In) { PackedRayCone = PackNormalToUInt32(In); }
	float3 GetRayDirection()		{ return UnpackNormalFromUInt32(PackedRayCone); }

	float3 GetWorldNormal()
	{
		float3 Result;
		Result.x = f16tof32(RadianceAndNormal[1] >> 16);
		Result.y = f16tof32(RadianceAndNormal[2]);
		Result.z = f16tof32(RadianceAndNormal[2] >> 16);
		return Result;
	}

	float4 GetCustomData()
	{
		float4 Result;
		Result.x = float(PackedCustomData & 0xFF);
		Result.y = float((PackedCustomData >> 8) & 0xFF);
		Result.z = float((PackedCustomData >> 16) & 0xFF);
		Result.w = float((PackedCustomData >> 24) & 0xFF);
		Result /= 255.0;
		return Result;
	}	

	float3 GetBaseColor()
	{
		float3 Result;
		Result.x = f16tof32(BaseColorAndOpacity[0]);
		Result.y = f16tof32(BaseColorAndOpacity[0] >> 16);
		Result.z = f16tof32(BaseColorAndOpacity[1]);
		return Result;
	}
	
	float3 GetWorldTangent()
	{
		float3 Result;
		Result.x = f16tof32(WorldTangentAndAnisotropy[0]);
		Result.y = f16tof32(WorldTangentAndAnisotropy[0] >> 16);
		Result.z = f16tof32(WorldTangentAndAnisotropy[1]);
		return Result;
	}

	float3 GetShadowVisibility()
	{
		return GetIndirectIrradiance();
	}

	void SetShadowVisibility(float3 ShadowVisibility)
	{
		SetIndirectIrradiance(ShadowVisibility);
	}

	uint GetLightIndex()
	{
		return PackedIndirectIrradiance[1] >> 16;
	}

	void SetLightIndex(uint LightIndex)
	{
		// NOTE: this assumes a maximum of 64k lights
		PackedIndirectIrradiance[1] = (LightIndex << 16) | (PackedIndirectIrradiance[1] & 0xFFFFu);
	}

	float GetOpacity()       { return f16tof32(BaseColorAndOpacity[1] >> 16); }
	void SetOpacity(float Opacity) { BaseColorAndOpacity[1] = f32tof16(Opacity) << 16 | (BaseColorAndOpacity[1] & 0xFFFF); }
	float GetMetallic()      { return float(MetallicAndSpecularAndRoughness & 0xFF) / 255.0f; }
	float GetSpecular()      { return float((MetallicAndSpecularAndRoughness >> 8) & 0xFF) / 255.0f; }
	float GetRoughness()     { return f16tof32(MetallicAndSpecularAndRoughness >> 16); }
	float GetIor()           { return f16tof32(IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask); }
	uint GetShadingModelID() { return (IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask >> 16) & 0xF; }
	uint GetBlendingMode()   { return (IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask >> 20) & 0x7; }
	uint GetFlags()          { return FlagsAndMipBias & 0xFF; }
	void SetFlags(uint Flags) { FlagsAndMipBias |= (Flags & 0xFF); }
	bool IsTwoSided()		 { return (GetFlags() & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_TWO_SIDED) != 0; }
	bool IsFrontFace()		 { return (GetFlags() & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0; }
	uint GetPrimitiveLightingChannelMask() { return (IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask >> 29) & 0x7; }
	float GetAnisotropy()    { return f16tof32(WorldTangentAndAnisotropy[1] >> 16); }
	bool IsValid()			 { return GetShadingModelID() != SHADINGMODELID_UNLIT; }

	float3 GetDiffuseColor() { return GetBaseColor() - GetBaseColor() * GetMetallic(); }
	float3 GetSpecularColor() { return ComputeF0(GetSpecular(), GetBaseColor(), GetMetallic()); }

	// Member function for (IsLumenPayload() == 1)
	void SetSceneInstanceIndex(uint GPUSceneInstanceIndex)
	{
		// Note: overwriting WorldTangentAndAnisotropy[1] so GetWorldTangent() won't work in Lumen
		WorldTangentAndAnisotropy[1] = GPUSceneInstanceIndex;
	}

	void SetGeometryNormal(float3 GeometryNormal)
	{
		PackedGeometryNormal = PackNormalToUInt32(GeometryNormal);
	}

	float3 GetGeometryNormal() 
	{ 
		return UnpackNormalFromUInt32(PackedGeometryNormal);
	}

	int GetSceneInstanceIndex() { return WorldTangentAndAnisotropy[1]; }

	// SceneInstanceIndex is not used in hit group shaders so reuse it to pass a random number
	void SetRandom(float E)					{ SetSceneInstanceIndex(asuint(E)); }
	float GetRandom()						{ return asfloat(GetSceneInstanceIndex()); }
};

FPackedMaterialClosestHitPayload PackRayTracingPayload(FMaterialClosestHitPayload Input, in FRayCone RayCone)
{
	FPackedMaterialClosestHitPayload Output = (FPackedMaterialClosestHitPayload)0;
	Output.FlagsAndMipBias = Input.Flags;
	Output.HitT = Input.HitT;
	Output.SetRayCone(RayCone);
	Output.RadianceAndNormal[0]  = f32tof16(Input.Radiance.x);
	Output.RadianceAndNormal[0] |= f32tof16(Input.Radiance.y) << 16;
	Output.RadianceAndNormal[1]  = f32tof16(Input.Radiance.z);
	Output.RadianceAndNormal[1] |= f32tof16(Input.WorldNormal.x) << 16;
	Output.RadianceAndNormal[2]  = f32tof16(Input.WorldNormal.y);
	Output.RadianceAndNormal[2] |= f32tof16(Input.WorldNormal.z) << 16;
	Output.BaseColorAndOpacity[0]  = f32tof16(Input.BaseColor.x);
	Output.BaseColorAndOpacity[0] |= f32tof16(Input.BaseColor.y) << 16;
	Output.BaseColorAndOpacity[1]  = f32tof16(Input.BaseColor.z);
	Output.BaseColorAndOpacity[1] |= f32tof16(Input.Opacity) << 16;
	Output.MetallicAndSpecularAndRoughness  = (uint(round(Input.Metallic * 255.0f)) & 0xFF);
	Output.MetallicAndSpecularAndRoughness |= (uint(round(Input.Specular * 255.0f)) & 0xFF) << 8;
	Output.MetallicAndSpecularAndRoughness |= f32tof16(Input.Roughness) << 16;
	Output.IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask  = f32tof16(Input.Ior);                              // 16 bits
	Output.IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask |= (Input.ShadingModelID & 0xF) << 16;               // 4 bits
	Output.IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask |= (Input.BlendingMode & 0x7) << 20;                 // 3 bits
	Output.IorAndShadingModelIDAndBlendingModeAndPrimitiveLightingChannelMask |= (Input.PrimitiveLightingChannelMask & 0x7) << 29; // 3 bits
	Output.PackedIndirectIrradiance[0]  = f32tof16(Input.IndirectIrradiance.x);
	Output.PackedIndirectIrradiance[0] |= f32tof16(Input.IndirectIrradiance.y) << 16;
	Output.PackedIndirectIrradiance[1]  = f32tof16(Input.IndirectIrradiance.z);
	int4 CustomData = round(Input.CustomData * 255);
	Output.PackedCustomData = CustomData.x | (CustomData.y << 8) | (CustomData.z << 16) | (CustomData.w << 24);
	Output.WorldTangentAndAnisotropy[0]  = f32tof16(Input.WorldTangent.x);
	Output.WorldTangentAndAnisotropy[0] |= f32tof16(Input.WorldTangent.y) << 16;
	Output.WorldTangentAndAnisotropy[1]  = f32tof16(Input.WorldTangent.z);
	Output.WorldTangentAndAnisotropy[1] |= f32tof16(Input.Anisotropy) << 16;

	return Output;
}

FMaterialClosestHitPayload UnpackRayTracingPayload(FPackedMaterialClosestHitPayload Input, FRayDesc Ray)
{
	FMaterialClosestHitPayload Output = (FMaterialClosestHitPayload)0;

	Output.HitT = Input.HitT;
	Output.RayCone            = Input.GetRayCone(); 
	Output.Radiance           = Input.GetRadiance();
	Output.WorldNormal        = Input.GetWorldNormal();
	Output.BaseColor          = Input.GetBaseColor();
	Output.Opacity	          = Input.GetOpacity();
	Output.Metallic	          = Input.GetMetallic();
	Output.Specular           = Input.GetSpecular();
    Output.Roughness          = Input.GetRoughness();
	Output.Ior                = Input.GetIor();
	Output.ShadingModelID     = Input.GetShadingModelID();
	Output.BlendingMode	      = Input.GetBlendingMode();
	Output.PrimitiveLightingChannelMask = Input.GetPrimitiveLightingChannelMask();
	Output.Flags              = Input.GetFlags();
	Output.IndirectIrradiance = Input.GetIndirectIrradiance();
	Output.CustomData         = Input.GetCustomData();
	Output.WorldTangent		  = Input.GetWorldTangent();
	Output.Anisotropy         = Input.GetAnisotropy();

	Output.DiffuseColor  = Output.BaseColor - Output.BaseColor * Output.Metallic;
	Output.SpecularColor = ComputeF0(Output.Specular, Output.BaseColor, Output.Metallic);
	Output.TranslatedWorldPos = Ray.Origin + Output.HitT * Ray.Direction;

	return Output;
}

#endif // SUBSTRATE_GBUFFER_FORMAT==1

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_RAYTRACING_MATERIAL)
CHECK_RT_PAYLOAD_SIZE(FPackedMaterialClosestHitPayload)
#endif

#endif // !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

///////////////////////////////////////////////////////////////////////////////////////////////////
// Decal parameters

// Must match EDecalsWriteFlags
// TODO: Move to a shared header?
#define DECAL_WRITE_BASE_COLOR_FLAG						(1 << 0)
#define DECAL_WRITE_NORMAL_FLAG							(1 << 1)
#define DECAL_WRITE_ROUGHNESS_SPECULAR_METALLIC_FLAG	(1 << 2)
#define DECAL_WRITE_EMISSIVE_FLAG						(1 << 3)

#define DECAL_USE_REVERSE_CULLING 						1

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_DECALS)
struct FDecalShaderPayload : FMinimalPayload
{
	// float HitT                          //  4 bytes
	float3 TranslatedWorldPos;             // 12 bytes
	uint2 PackedBaseColor;                 //  8 bytes half4
	uint2 PackedMetallicSpecularRoughness; //  8 bytes - half4
	uint2 PackedWorldNormal;               //  8 bytes - half4
	uint2 PackedEmissive;                  //  8 bytes - half3 (alpha is not used)
	                                       // 48 bytes total

	float4 GetBaseColor() {
		return float4(UnpackFloat2FromUInt(PackedBaseColor.x),
		              UnpackFloat2FromUInt(PackedBaseColor.y));
	}

	void SetBaseColor(float4 BaseColor) {
		PackedBaseColor.x = PackFloat2ToUInt(BaseColor.rg);
		PackedBaseColor.y = PackFloat2ToUInt(BaseColor.ba);
	}

	float4 GetMetallicSpecularRoughness() {
		return float4(UnpackFloat2FromUInt(PackedMetallicSpecularRoughness.x),
		              UnpackFloat2FromUInt(PackedMetallicSpecularRoughness.y));
	}

	void SetMetallicSpecularRoughness(float4 MetallicSpecularRoughness) {
		PackedMetallicSpecularRoughness.x = PackFloat2ToUInt(MetallicSpecularRoughness.rg);
		PackedMetallicSpecularRoughness.y = PackFloat2ToUInt(MetallicSpecularRoughness.ba);
	}

	float4 GetWorldNormal() {
		return float4(UnpackFloat2FromUInt(PackedWorldNormal.x),
		              UnpackFloat2FromUInt(PackedWorldNormal.y));
	}

	void SetWorldNormal(float4 WorldNormal) {
		PackedWorldNormal.x = PackFloat2ToUInt(WorldNormal.xy);
		PackedWorldNormal.y = PackFloat2ToUInt(WorldNormal.zw);
	}

	float3 GetEmissive() {
		return float3(UnpackFloat2FromUInt(PackedEmissive.x),
		              f16tof32(PackedEmissive.y));
	}

	void SetEmissive(float3 Emissive) {
		PackedEmissive.x = PackFloat2ToUInt(Emissive.xy);
		PackedEmissive.y = f32tof16(Emissive.z);
	}
};
CHECK_RT_PAYLOAD_SIZE(FDecalShaderPayload)


FDecalShaderPayload InitDecalShaderPayload(float3 TranslatedWorldPos)
{
	FDecalShaderPayload Payload = (FDecalShaderPayload)0;
	Payload.TranslatedWorldPos = TranslatedWorldPos;
	// set alpha=1 on all fields
	Payload.PackedBaseColor.y = f32tof16(1.0) << 16;
	Payload.PackedMetallicSpecularRoughness.y = f32tof16(1.0) << 16;
	Payload.PackedWorldNormal.y = f32tof16(1.0) << 16;
	return Payload;
}

#endif


///////////////////////////////////////////////////////////////////////////////////////////////////
// Tracing util functions

#if !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#if COMPUTESHADER
#include "TraceRayInline.ush"
#endif // COMPUTESHADER

void TraceVisibilityRayPacked(
	inout FPackedMaterialClosestHitPayload PackedPayload,
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in FRayDesc Ray)
{
	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_SHADOW;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;
	const uint MissShaderIndex = 0;

	// By enabling minimal payload mode all other payload information is ignored, meaning these functions need no payload inputs
	PackedPayload.SetMinimalPayloadMode();
	PackedPayload.HitT = 0;

	// Trace the ray

#if COMPUTESHADER

	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();

	FTraceRayInlineResult TraceRayResult = TraceRayInline(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		Ray.GetNativeDesc(),
		TraceRayInlineContext
	);
	PackedPayload.HitT = TraceRayResult.HitT;

#else

	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);

#endif // COMPUTESHADER
}

FMinimalPayload TraceVisibilityRay(
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in FRayDesc Ray)
{
	FPackedMaterialClosestHitPayload PackedPayload = (FPackedMaterialClosestHitPayload)0;
	
	PackedPayload.SetFlags(RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY);

	//Have shadow ray behaves the same to Lumen, not tracing translucent materials.
	PackedPayload.SetIgnoreTranslucentMaterials();

	TraceVisibilityRayPacked(PackedPayload, TLAS, RayFlags, InstanceInclusionMask, Ray);

	// Unpack the payload

	FMinimalPayload MinimalPayload = (FMinimalPayload)0;

	// In theory this unpacking setp is not needed as FPackedMaterialClosestHitPayload derives from FMinimalPayload,
	// but the compiler currently dislikes a direct cast between them. Additionally in the future if HitT is ever packed
	// differently and the FMinimalPayload is not directly inherited from this will need to change.
	MinimalPayload.HitT = PackedPayload.HitT;

	return MinimalPayload;
}

#if !COMPUTESHADER

///////////////////////////////////////////////////////////////////////////////////////////////////
// Tracing translucent visibility ray functions

struct FTranslucentMinimalPayload : FMinimalPayload
{
	float3 ShadowVisibility; // What fraction of the ray's radiance is visible? (1.0 - fully visible, 0.0 - fully shadowed)
};

FTranslucentMinimalPayload TraceTranslucentVisibilityRay(
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in FRayDesc Ray)
{
	FPackedMaterialClosestHitPayload PackedPayload = (FPackedMaterialClosestHitPayload)0;
	PackedPayload.SetShadowRay();
	PackedPayload.SetMinimalPayloadMode();
	PackedPayload.HitT = 0; // default value in case we don't run the miss shader
	PackedPayload.SetShadowVisibility(1.0);

	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_SHADOW;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;
	const uint MissShaderIndex = 0;

	// Trace the ray
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);

	// Unpack the payload
	FTranslucentMinimalPayload MinimalPayload = (FTranslucentMinimalPayload)0;

	MinimalPayload.HitT = PackedPayload.HitT;
	// only read the throughput if we got a miss (ignored all hits and ran the miss shader which reset HitT)
	// If we registered a hit, it means that either we ran the closest hit shader which set a valid distance,
	// or we never ran the miss shader and kept the HitT value from above (or one set by the AHS before the ray was terminated)
	// in both cases we know the shadow visibility should be counted as zero.
	MinimalPayload.ShadowVisibility = PackedPayload.IsMiss() ? PackedPayload.GetShadowVisibility() : 0.0;

	return MinimalPayload;
}

void TraceMaterialRayPacked(
	inout FPackedMaterialClosestHitPayload Payload,
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in FRayDesc Ray,
	// Payload Inputs
	inout FRayCone RayCone,
	in bool bEnableSkyLightContribution,
	in bool bIsCameraRay)
{
	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_MATERIAL;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;
	const uint MissShaderIndex = 0;

	// Set payload inputs

	Payload.SetRayCone(RayCone);

	if (bEnableSkyLightContribution)
	{
		// Enable Sky Light contribution in the payload if requested
		Payload.SetEnableSkyLightContribution();
	}

	if (bIsCameraRay)
	{
		Payload.SetCameraRay();
	}


	// Trace the ray

	Payload.HitT = 0;
	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		Payload);

	RayCone = Payload.GetRayCone();
}

FMaterialClosestHitPayload TraceMaterialRay(
	in RaytracingAccelerationStructure TLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in FRayDesc Ray,
	// Payload Inputs
	inout FRayCone RayCone,
	in bool bEnableSkyLightContribution,
	in bool bIgnoreTranslucentMaterials,
	in bool bIsCameraRay)
{
	const uint RayContributionToHitGroupIndex = RAY_TRACING_SHADER_SLOT_MATERIAL;
	const uint MultiplierForGeometryContributionToShaderIndex = RAY_TRACING_NUM_SHADER_SLOTS;
	const uint MissShaderIndex = 0;

	FPackedMaterialClosestHitPayload PackedPayload = (FPackedMaterialClosestHitPayload)0;

	// Set payload inputs

	PackedPayload.SetRayCone(RayCone);

	if (bEnableSkyLightContribution)
	{
		// Enable Sky Light contribution in the payload if requested
		PackedPayload.SetEnableSkyLightContribution();
	}
	
	if (bIgnoreTranslucentMaterials)
	{
		PackedPayload.SetIgnoreTranslucentMaterials();
	}

	if (bIsCameraRay)
	{
		PackedPayload.SetCameraRay();
	}

	// Trace the ray

	TraceRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RayContributionToHitGroupIndex,
		MultiplierForGeometryContributionToShaderIndex,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload);

	// Unpack the payload

	FMaterialClosestHitPayload Payload = UnpackRayTracingPayload(PackedPayload, Ray);

	RayCone = Payload.GetRayCone();

	return Payload;
}

#endif // !COMPUTESHADER

#endif // !COMPUTESHADER || PLATFORM_SUPPORTS_INLINE_RAY_TRACING

uint GetGPUSceneInstanceIndex(uint PrimitiveIndex, uint InstanceIndex)
{
#if VF_USE_PRIMITIVE_SCENE_DATA
	FPrimitiveSceneData PrimData = GetPrimitiveData(PrimitiveIndex);
	// Handle ray tracing auto instancing which can cause PrimitiveInstanceIndex to be above 0 for meshes with a single instance
	if (PrimData.NumInstanceSceneDataEntries == 1)
	{
		InstanceIndex = 0;
	}

	return PrimData.InstanceSceneDataOffset + InstanceIndex;
#else

	return 0;
#endif
}

#endif // RAYTRACINGCOMMON_USH_INCLUDED // Workarround for UE-66460
