// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
    RayTracingCommonMetal.ush: Metal-specific ray tracing overrides.
==============================================================================*/

#pragma once

#define OVERRIDE_PLATFORM_RAYTRACING_INSTANCE_DESCRITOR 1

#define PLATFORM_RAYTRACING_INSTANCE_FLAG_NONE                                0   // MTLAccelerationStructureInstanceOptionNone
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_CULL_DISABLE               0x1 // MTLAccelerationStructureInstanceOptionDisableTriangleCulling
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_TRIANGLE_FRONT_COUNTERCLOCKWISE     0x2 // MTLAccelerationStructureInstanceOptionTriangleFrontFacingWindingCounterClockwise
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_OPAQUE                        0x4 // MTLAccelerationStructureInstanceOptionOpaque
#define PLATFORM_RAYTRACING_INSTANCE_FLAG_FORCE_NON_OPAQUE                    0x8 // MTLAccelerationStructureInstanceOptionNonOpaque

// Mirrors IndirectAccelerationStructureInstanceDescriptor:
struct FPlatformRayTracingInstanceDescriptor
{
    float Transform[4][3]; // MTLPackedFloat4x3 transformationMatrix;
    uint Options;  //  MTLAccelerationStructureInstanceOptions options;
    uint InstanceMask; // uint32_t mask;
    uint intersectionFunctionTableOffset; // uint32_t intersectionFunctionTableOffset;
    uint InstanceID; // uint32_t userID;
	uint2 AccelerationStructure; // uint2 accelerationStructureIndex;
	
	uint GetInstanceId()
	{
		return InstanceID;
	}

	uint GetMask()
	{
		return InstanceMask;
	}

	uint GetInstanceContributionToHitGroupIndex()
	{
		return 0;
	}

	uint GetFlags()
	{
		return Options;
	}
};

void SetRayTracingInstanceTransform(inout FPlatformRayTracingInstanceDescriptor Desc, float3x4 LocalToWorld)
{
    for (uint j = 0; j < 4; j++)
        for (uint i = 0; i < 3; i++)
            Desc.Transform[j][i] = LocalToWorld[i][j];
}

uint GetRayTracingInstanceDescriptorInstanceId(FPlatformRayTracingInstanceDescriptor Desc)
{
    return Desc.GetInstanceId();
}

uint GetRayTracingInstanceDescriptorMask(FPlatformRayTracingInstanceDescriptor Desc)
{
    return Desc.GetMask();
}

uint GetRayTracingInstanceDescriptorInstanceContributionToHitGroupIndex(FPlatformRayTracingInstanceDescriptor Desc)
{
    return Desc.GetInstanceContributionToHitGroupIndex();
}

uint GetRayTracingInstanceDescriptorFlags(FPlatformRayTracingInstanceDescriptor Desc)
{
    return Desc.GetFlags();
}

FPlatformRayTracingInstanceDescriptor BuildPlatformRayTracingInstanceDesc(uint InstanceMask, uint InstanceId, uint InstanceFlags, uint InstanceContributionToHitGroupIndex, float3x4 LocalToWorld, uint2 AccelerationStructureAddress)
{
    FPlatformRayTracingInstanceDescriptor Desc;
    Desc.Options = InstanceFlags;
    Desc.InstanceMask = InstanceMask;
	// TODO: Carl - Fix procedural when no custom intersection is present
	Desc.intersectionFunctionTableOffset = 0;
    Desc.AccelerationStructure = AccelerationStructureAddress;
    Desc.InstanceID = InstanceId;
    SetRayTracingInstanceTransform(Desc, LocalToWorld);

    return Desc;
}
