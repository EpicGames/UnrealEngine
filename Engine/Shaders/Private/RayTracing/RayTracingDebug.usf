// Copyright Epic Games, Inc. All Rights Reserved.

#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
#define SUBSTRATE_SSS_TRANSMISSION 0				// Disable for now, as the SSS profile texture need to be bound
#define SUBSTRATE_COMPLEXSPECIALPATH 1				// Allow to read data from complex special path
#define SUBSTRATE_INLINE_SHADING 0

#define RAYTRACING_DEBUG_INLINE COMPUTESHADER

#include "/Engine/Shared/RayTracingDebugTypes.h"
#include "/Engine/Shared/RayTracingTypes.h"
#include "../Common.ush"
#include "../ShadingCommon.ush"
#include "../ColorMap.ush"
#include "../Hash.ush"
#include "../Visualization.ush"
#include "../ShaderPrint.ush"
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateEvaluation.ush"
#include "../Substrate/SubstrateVisualizeCommon.ush"
#include "../Nanite/NaniteRayTrace.ush"
#include "/Engine/Shared/SubstrateVisualizeDefinitions.h"

#include "RayTracingDebugUtils.ush"
#include "RayTracingLightCullingCommon.ush"

#include "/Engine/Shared/RaytracingDebugDefinitions.h"

RWTexture2D<float4> Output;
RWTexture2D<float> OutputDepth;

RaytracingAccelerationStructure TLAS;

#if RAYTRACING_DEBUG_INLINE
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata;
#endif

uint VisualizationMode;
uint PickerDomain;
uint OpaqueOnly;
float TimingScale;

float TriangleHitCountMaxThreshold;
float TriangleHitCountPerInstanceMaxThreshold;
uint TopKMostHitInstances;
uint NumTotalInstances;

StructuredBuffer<FRayTracingInstanceExtraData> InstancesExtraData;
StructuredBuffer<FRayTracingInstanceDebugData> InstancesDebugData;
StructuredBuffer<FPlatformRayTracingInstanceDescriptor> InstanceBuffer;
StructuredBuffer<FRayTracingPickingFeedback> PickingBuffer;

#if (USE_DEBUG_CHS == 0) && (RAYTRACING_DEBUG_INLINE == 0) // add functions that require shaded payload

float4 AnisoDebugColor(float InAniso)
{
	// Follow same mapping than raster buffer visualization (BP located in Engine/Contents/Anisotropy)
	float AniG = saturate(InAniso);
	float AniB = saturate(-1.0 * InAniso);
	return float4(0.0, pow(AniG, 2.2), pow(AniB, 2.2), 0.0f);
}

float4 RoughnessDebugColor(float InRoughness)
{
	return float4(pow(InRoughness, 2.2f).xxx, 0.0f);
}

float4 DebugRayTracingMaterial(uint2 PixelCoord, FRayDesc Ray)
{
	FRayCone RayCone = (FRayCone)0;
	RayCone.SpreadAngle = View.EyeToPixelSpreadAngle;

	const uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
	const uint InstanceInclusionMask = OpaqueOnly ? (RAY_TRACING_MASK_OPAQUE | RAY_TRACING_MASK_HAIR_STRANDS) : RAY_TRACING_MASK_ALL;
	const bool bEnableSkyLightContribution = true;
	const bool bIgnoreTranslucentMaterials = false;
	const bool bCameraRay = true;
	
	FMaterialClosestHitPayload Payload = (FMaterialClosestHitPayload)0;

#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
	uint TimestampDiff = 0;

	if (VisualizationMode == RAY_TRACING_DEBUG_VIZ_TIMING_ANY_HIT)
	{
		FTimestamp TimeBegin = GetShaderTimestamp();

		TraceVisibilityRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			Ray);

		FTimestamp TimeEnd = GetShaderTimestamp();

		TimestampDiff = ShaderTimestampDiff(TimeBegin, TimeEnd);
	}
	else
#endif
	{
#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
		FTimestamp TimeBegin = GetShaderTimestamp();
#endif

		Payload = TraceMaterialRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			Ray,
			RayCone,
			bEnableSkyLightContribution,
			bIgnoreTranslucentMaterials,
			bCameraRay);

#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
		FTimestamp TimeEnd = GetShaderTimestamp();

		TimestampDiff = ShaderTimestampDiff(TimeBegin, TimeEnd);
#endif
	}

	if (Payload.IsHit())
	{
		// 1. Non-material dependent debug view
		switch (VisualizationMode)
		{
		case RAY_TRACING_DEBUG_VIZ_RADIANCE:				return float4(Payload.Radiance, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_OPACITY:					return float4((1.0f - Payload.Opacity).xxx, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_BLENDING_MODE:			return float4(Payload.BlendingMode.xxx, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_LIGHTING_CHANNEL_MASK:	return float4(Payload.PrimitiveLightingChannelMask & 0x1, Payload.PrimitiveLightingChannelMask & 0x2, Payload.PrimitiveLightingChannelMask & 0x4, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_INDIRECT_IRRADIANCE:		return float4(Payload.IndirectIrradiance, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_WORLD_POSITION:			return float4(Payload.TranslatedWorldPos, 0.0f);
		case RAY_TRACING_DEBUG_VIZ_HITKIND:					return Payload.IsFrontFace() ? float4(0.0, 1.0, 0.0, 0.0f) : float4(1.0, 0.0, 0.0, 0.0f);

		#if !VULKAN_PROFILE_SM6
		// :todo-jn:  disabled until SPIRV codegen issue can be found (breaks all other debug modes when enabled). See UE-218212.
		case RAY_TRACING_DEBUG_VIZ_LIGHT_GRID_COUNT:	    return float4(FCulledLightList::Create(Payload.TranslatedWorldPos).Visualize(1) * abs(dot(Payload.WorldNormal, Ray.Direction)), 0.0f);
		#endif

		#if PLATFORM_SUPPORTS_SHADER_TIMESTAMP
		case RAY_TRACING_DEBUG_VIZ_TIMING_ANY_HIT:
		case RAY_TRACING_DEBUG_VIZ_TIMING_MATERIAL:				return float4(ColorMapInferno(float(TimestampDiff) * TimingScale), 0.0f);
		#endif // PLATFORM_SUPPORTS_SHADER_TIMESTAMP
		}

		// 2. Material dependent view mode
	#if SUBSTRATE_GBUFFER_FORMAT==1
		FSubstrateAddressing SubstrateAddressing   = GetSubstratePixelDataByteOffset(0, 0, 0);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Payload.SubstrateData, SubstrateAddressing, Payload.SubstrateData);

		BRANCH
		if (VisualizationMode == RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA)
		{
			uint2 CursorPixelCoord = GetCursorPos() * View.ViewResolutionFraction;
			FShaderPrintContext Ctx = InitShaderPrintContext(all(CursorPixelCoord == PixelCoord), uint2(50, 50));
			if (Ctx.bIsActive)
			{
				//const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Payload.SubstrateData, SubstrateAddressing, SubstratePixelHeader); // Take the parameter of the first BSDF
				const float3 TranslatedWorldPosition = Ray.Origin + Payload.HitT * Ray.Direction;
				const float3 V = -normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
				const uint ClosureIndex = 0;
				SubstratePrintMaterialProperties(Ctx, PixelCoord, TranslatedWorldPosition, V, ClosureIndex, Payload.SubstrateData);

				SerializeSubstratePixelDebugDataEntry(PixelCoord, Payload.SubstrateData, View.FrameNumber);

			}
			return float4(0, 0, 0, 1);
		}
		else if (SubstratePixelHeader.ClosureCount > 0)
		{
			const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Payload.SubstrateData, SubstrateAddressing, SubstratePixelHeader); // Take the parameter of the first BSDF
			const float3x3 TangentBasis = SubstrateGetBSDFSharedBasis(SubstratePixelHeader, BSDF, SubstrateAddressing);
			switch (VisualizationMode)
			{
			case RAY_TRACING_DEBUG_VIZ_GBUFFER_AO:		return float4(SubstrateGetIrradianceAndAO(SubstratePixelHeader).MaterialAO.xxx, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SHADING_MODEL:	return float4(GetShadingModelColor(SubstrateGetLegacyShadingModels(BSDF)), 0.0f);
			case RAY_TRACING_DEBUG_VIZ_BASE_COLOR:		return float4(SubstrateGetBSDFBaseColor(BSDF), 0.0f);
			case RAY_TRACING_DEBUG_VIZ_DIFFUSE_COLOR:	return float4(SubstrateGetBSDFDiffuseColor(BSDF), 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SPECULAR_COLOR:	return float4(SubstrateGetBSDFSpecularF0(BSDF).xxx, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_METALLIC:		return float4(SubstrateGetBSDFMetallic(BSDF).xxx, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SPECULAR:		return float4(SubstrateGetBSDFSpecular(BSDF).xxx, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_ROUGHNESS:		return RoughnessDebugColor(SubstrateGetBSDFRoughness(BSDF));
			case RAY_TRACING_DEBUG_VIZ_IOR:				return float4(1, 1, 1, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_CUSTOM_DATA:		return float4(0, 0, 0, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL:	return float4(TangentBasis[2] * 0.5 + 0.5, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_WORLD_TANGENT:	return float4(TangentBasis[0] * 0.5 + 0.5, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_ANISOTROPY:		return AnisoDebugColor(SubstrateGetBSDFAnisotropy(BSDF));
			}
		}
	#else // SUBSTRATE_GBUFFER_FORMAT==1
		switch (VisualizationMode)
		{
			case RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL:	return float4(Payload.WorldNormal * 0.5 + 0.5, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_GBUFFER_AO:		return float4(Payload.GBufferAO, Payload.GBufferAO, Payload.GBufferAO, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SHADING_MODEL:	return float4(GetShadingModelColor(Payload.ShadingModelID), 0.0f);
			case RAY_TRACING_DEBUG_VIZ_BASE_COLOR:		return float4(Payload.BaseColor.rgb, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_DIFFUSE_COLOR:	return float4(Payload.DiffuseColor.rgb, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SPECULAR_COLOR:	return float4(Payload.SpecularColor.rgb, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_METALLIC:		return float4(Payload.Metallic, Payload.Metallic, Payload.Metallic, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_SPECULAR:		return float4(Payload.Specular, Payload.Specular, Payload.Specular, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_ROUGHNESS:		return RoughnessDebugColor(Payload.Roughness);
			case RAY_TRACING_DEBUG_VIZ_IOR:				return float4(Payload.Ior, Payload.Ior, Payload.Ior, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_CUSTOM_DATA:		return float4(Payload.CustomData.xyz, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_WORLD_TANGENT:	return float4(Payload.WorldTangent * 0.5 + 0.5, 0.0f);
			case RAY_TRACING_DEBUG_VIZ_ANISOTROPY:		return AnisoDebugColor(Payload.Anisotropy);
		}
	#endif // SUBSTRATE_GBUFFER_FORMAT==1
	}

	// HitT is packed into the alpha for far field compositing
	return float4(0, 0, 0, -1.0);
}

#else // USE_DEBUG_CHS

// Add functions that use the debug payload only

float4 DebugRayTracingInstance(FRayDesc Ray, out float OutDepth)
{
	float4 Result = float4(0, 0, 0, 1);

	FRayTracingDebugPayload Payload = (FRayTracingDebugPayload)0;
	Payload.SetMiss();

	uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
	const uint InstanceInclusionMask = (OpaqueOnly ? RAY_TRACING_MASK_OPAQUE : RAY_TRACING_MASK_ALL);
	if (VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT)
	{
		RayFlags |= RAY_FLAG_FORCE_NON_OPAQUE;
	}
	else
	{
		RayFlags |= RAY_FLAG_FORCE_OPAQUE;
	}

#if RAYTRACING_DEBUG_INLINE
	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS // && NANITE_RAY_TRACING
	TraceRayInlineContext.bOutputClusterID = UsingCLAS;
#endif

	FTraceRayInlineResult TraceRayResult = TraceRayInline(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		Ray.GetNativeDesc(),
		TraceRayInlineContext
	);
	
	Payload.HitT = TraceRayResult.HitT;

	const uint GPUSceneInstanceId = TraceRayResult.InstanceID;

	const FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);

	Payload.GPUSceneInstanceId = GPUSceneInstanceId;
	Payload.TriangleIndex = TraceRayResult.PrimitiveIndex;

	float3 WorldNormal;
#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS
	WorldNormal = TraceRayResult.WorldGeometryNormal;
#elif PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
	float3 LocalEdges[2];
	// NANITE_RAY_TRACING
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceSceneData.PrimitiveId);

#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	if (UsingCLAS && PrimitiveData.NaniteResourceID != uint(-1))
	{
		const uint AssemblyTransformIndex = NANITE_MAX_ASSEMBLY_TRANSFORMS;

		const uint PageIndex = TraceRayResult.ClusterID & NANITE_MAX_GPU_PAGES_MASK;
		const uint ClusterIndex = (TraceRayResult.ClusterID >> NANITE_MAX_GPU_PAGES_BITS) & NANITE_MAX_CLUSTERS_PER_PAGE_MASK;

		GetNaniteTriangleEdges(
			PageIndex,
			ClusterIndex,
			TraceRayResult.PrimitiveIndex,
			AssemblyTransformIndex,
			PrimitiveData.NaniteAssemblyTransformOffset,
			LocalEdges[0],
			LocalEdges[1]);
	}
	else
#endif // PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	if (PrimitiveData.NaniteRayTracingDataOffset != uint(-1))
	{
		const uint HitGroupFirstPrimitive = GetInlineRayTracingHitGroupFirstPrimitive(
			RayTracingSceneMetadata,
			TraceRayResult.InstanceContributionToHitGroupIndex,
			RAY_TRACING_NUM_SHADER_SLOTS,
			TraceRayResult.GeometryIndex
		);

		GetNaniteTriangleEdges(PrimitiveData.NaniteRayTracingDataOffset + HitGroupFirstPrimitive + TraceRayResult.PrimitiveIndex, PrimitiveData.NaniteAssemblyTransformOffset, LocalEdges[0], LocalEdges[1]);
	}
	else
	{
		FTriangleBaseAttributes Attributes = LoadInlineRayTracingTriangleAttributes(
			RayTracingSceneMetadata,
			TraceRayResult.InstanceContributionToHitGroupIndex,
			RAY_TRACING_NUM_SHADER_SLOTS,
			TraceRayResult.GeometryIndex,
			TraceRayResult.PrimitiveIndex
		);
		LocalEdges[0] = Attributes.LocalPositions[1] - Attributes.LocalPositions[0];
		LocalEdges[1] = Attributes.LocalPositions[2] - Attributes.LocalPositions[0];
	}

	const float3 LocalNormal = cross(LocalEdges[1], LocalEdges[0]);
	WorldNormal = normalize(mul(LocalNormal, (float3x3)TraceRayResult.WorldToObject3x4));
#else // !PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
	WorldNormal = 0.0f;
#endif // !PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
	Payload.WorldNormal = WorldNormal;
	Payload.InstanceIndex = TraceRayResult.InstanceIndex;
	Payload.GeometryIndex = TraceRayResult.GeometryIndex;
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	Payload.ClusterId = TraceRayResult.ClusterID;
#endif

	if (TraceRayResult.bIsFrontFace)
	{
		Payload.SetFrontFace();
	}
#else
	TraceRay(
		TLAS,
		RayFlags  /*RayFlags*/,
		InstanceInclusionMask /*InstanceInclusionMask*/,
		0 /*RayContributionToHitGroupIndex*/,
		RAY_TRACING_NUM_SHADER_SLOTS /*MultiplierForGeometryContributionToShaderIndex*/,
		0 /*MissShaderIndex*/,
		Ray.GetNativeDesc(),
		Payload);
#endif

	Payload.WorldNormal = Payload.IsFrontFace() ? Payload.WorldNormal : -Payload.WorldNormal;

	if (Payload.IsHit())
	{
		const float3 InstanceColor = IntToColor(Payload.GPUSceneInstanceId);
		const float3 GeometryColor = IntToColor(MurmurAdd(Payload.GPUSceneInstanceId, Payload.GeometryIndex));
		const float3 TriangleColor = IntToColor(MurmurAdd(Payload.GPUSceneInstanceId, Payload.TriangleIndex));

#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
		const uint PageIndex = Payload.ClusterId & NANITE_MAX_GPU_PAGES_MASK;
		const uint ClusterIndex = (Payload.ClusterId >> NANITE_MAX_GPU_PAGES_BITS) & NANITE_MAX_CLUSTERS_PER_PAGE_MASK;
		const float3 ClusterColor = IntToColor(MurmurAdd(PageIndex, ClusterIndex)); // match NaniteVisualize.usf
#endif

		const FRayTracingInstanceExtraData InstanceExtraData = InstancesExtraData[Payload.InstanceIndex];
		const FRayTracingInstanceDebugData InstanceDebugData = InstancesDebugData[InstanceExtraData.SceneInstanceIndex];

		switch (VisualizationMode)
		{
		case RAY_TRACING_DEBUG_VIZ_INSTANCES:
			Result.rgb = InstanceColor;
			break;
		case RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL:
			Result.rgb = float3(Payload.WorldNormal * 0.5 + 0.5);
			break;
		case RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT:
			Result.rgb = ColorMapTurbo(Payload.TriangleHitCountPerRay / TriangleHitCountMaxThreshold);
			break;
		#if !VULKAN_PROFILE_SM6
		// :todo-jn:  disabled until SPIRV codegen issue can be found (breaks all other debug modes when enabled). See UE-218215.
		case RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE:
			Result.rgb = ColorMapTurbo(RayTracingDebugHitStatsUniformBuffer.HitStatsOutput[Payload.InstanceIndex].Count / (float)TriangleHitCountPerInstanceMaxThreshold);
			break;
		#endif
		case RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP:
			Result.rgb = 1.0; // We only use fake lighting as instance heatmap is blended in
			break;
		case RAY_TRACING_DEBUG_VIZ_TRIANGLES:
			Result.rgb = TriangleColor;
			break;
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
		case RAY_TRACING_DEBUG_VIZ_CLUSTERS:
			Result.rgb = ClusterColor;
			break;
#endif
		case RAY_TRACING_DEBUG_VIZ_DYNAMIC_INSTANCES:
			Result.rgb = InstanceDebugData.Flags ? float3(0.25f, 0.50f, 0.25f) : float3(0.50f, 0.25f, 0.25f);
			break;
		case RAY_TRACING_DEBUG_VIZ_PROXY_TYPE:
			Result.rgb = IntToColor(InstanceDebugData.ProxyHash);
			break;
		case RAY_TRACING_DEBUG_VIZ_HITKIND:
			Result.rgb = Payload.IsFrontFace() ? float3(0.0, 1.0, 0.0) : float3(1.0, 0.0, 0.0);
			break;
		default:
			Result.rgb = 0.5f;
			break;
		}

		if (VisualizationMode == RAY_TRACING_DEBUG_VIZ_PICKER)
		{
			const float ColorScale = 0.1f;
			const float BrighterColorScale = 0.2f;
			const float3 PickedColor = float3(1.0f, 0.0f, 1.0f);

			const bool bSameTriangle = (PickingBuffer[0].TriangleIndex == Payload.TriangleIndex);
			const bool bSameSegment = (PickingBuffer[0].GeometryIndex == Payload.GeometryIndex);
			const bool bSameInstance = (PickingBuffer[0].InstanceIndex == Payload.InstanceIndex);

			FPlatformRayTracingInstanceDescriptor InstanceDescriptor = InstanceBuffer[Payload.InstanceIndex];
			uint Flags = TranslateRayTracingPlatformInstanceFlags(GetRayTracingInstanceDescriptorFlags(InstanceDescriptor));
			uint Mask = GetRayTracingInstanceDescriptorMask(InstanceDescriptor);

			switch (PickerDomain)
			{
			case RAY_TRACING_DEBUG_PICKER_DOMAIN_TRIANGLE:
				Result.rgb = bSameInstance ? (bSameSegment && bSameTriangle ? PickedColor : TriangleColor * BrighterColorScale) : TriangleColor * ColorScale;
				break;				
			case RAY_TRACING_DEBUG_PICKER_DOMAIN_SEGMENT:
				Result.rgb = bSameInstance ? (bSameSegment ? PickedColor : GeometryColor * BrighterColorScale) : GeometryColor * ColorScale;
				break;
			case RAY_TRACING_DEBUG_PICKER_DOMAIN_INSTANCE:
				Result.rgb = bSameInstance ? PickedColor : InstanceColor * ColorScale;
				break;
			case RAY_TRACING_DEBUG_PICKER_DOMAIN_FLAGS:
				Result.rgb = (PickingBuffer[0].Flags == Flags ? BrighterColorScale : ColorScale) * IntToColor(Flags + 1);
				break;
			case RAY_TRACING_DEBUG_PICKER_DOMAIN_MASK:
				Result.rgb = (PickingBuffer[0].Mask == Mask ? BrighterColorScale : ColorScale) * IntToColor(Mask);
				break;
			}
		}

		// apply some shading based on the primitive normal (map dot product result from [0,1] to [0.75,1] unless we are showing instance heatmap then just use [0,1]).
		const float FakeLighting = dot(Payload.WorldNormal, normalize(float3(1.0f, 1.0f, 1.0f)));
		const float FakeLightingScale = (VisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP) ? 0.5f : 0.3f;
		const float FakeLightingBias = (VisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP) ? 0.5f : 0.7f;
		const int NoFakeLighting = (VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT) || (VisualizationMode == RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE) || (VisualizationMode == RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL);

		if (!NoFakeLighting)
		{
			Result.rgb *= FakeLighting * FakeLightingScale + FakeLightingBias;
		}

		float3 TranslatedWorldPosition = Ray.Origin + Payload.HitT * Ray.Direction;
		float4 ClipPosition = mul(float4(TranslatedWorldPosition, 1.0), View.TranslatedWorldToClip);
		OutDepth = ClipPosition.z / ClipPosition.w;
		Result.a = 0.0;
	}
	else
	{
		Result = float4(0, 0, 0, -1.0);
		OutDepth = 0.0;
	}

	return Result;
}
#endif


RAY_TRACING_ENTRY_RAYGEN_OR_INLINE(RayTracingDebugMain)
{
	uint2 PixelCoord = DispatchThreadIndex.xy + View.ViewRectMin.xy;

	float2 RenderTargetUV = (float2(PixelCoord) + .5f) * View.BufferSizeAndInvSize.zw;

	FRayDesc Ray = CreatePrimaryRay(RenderTargetUV);
	
	float OutDepth = 0.0f;

	bool bWriteColorOuput = VisualizationMode != RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA;
	bool bWriteDepthOutput = VisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP;
#if USE_DEBUG_CHS || RAYTRACING_DEBUG_INLINE
	float4 Result = DebugRayTracingInstance(Ray, OutDepth);
#else // USE_DEBUG_CHS
	float4 Result = DebugRayTracingMaterial(PixelCoord, Ray);
#endif // USE_DEBUG_CHS

	// When the visualization mode does not require exposure, we disable it via RaytracingDebugViewModeNeedsTonemapping which makes this factor 1.0
	Result.rgb *= View.PreExposure;

	if (!IsFinite(Result.r) || !IsFinite(Result.g) || !IsFinite(Result.b))
	{
		float T = frac(View.RealTime);
		if (T > 0.5)
		{
			Result.rgb = float3(1,0,1);
		}
	}
	
	if (bWriteColorOuput)
	{
		Output[PixelCoord] = float4(Result.rgb, 1.0f);
	}

	if (bWriteDepthOutput)
	{
		OutputDepth[PixelCoord] = OutDepth;
	}
}
