// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Hash.ush"

#include "RayTracingCommon.ush"
#include "RayTracingDebugUtils.ush"
#include "RayTracingHitGroupCommon.ush"

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingDebugMainCHS,
	FRayTracingDebugPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.HitT = RayTCurrent();

	const uint GPUSceneInstanceId = GetInstanceUserData();

	const FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);

	Payload.GPUSceneInstanceId = GPUSceneInstanceId;

	Payload.TriangleIndex = PrimitiveIndex();

	Payload.WorldNormal = GetGeometryNormalFromTriangleBaseAttributes(PrimitiveIndex(), GetClusterID());
	Payload.InstanceIndex = InstanceIndex();
	Payload.GeometryIndex = GeometryIndex();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	Payload.ClusterId = GetClusterID();
#endif
	
	if (HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE)
	{
		Payload.SetFrontFace();
	}
}

RAY_TRACING_ENTRY_ANY_HIT(RayTracingDebugAHS,
	FRayTracingDebugPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.TriangleHitCountPerRay += 1;
}
