// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "RayTracingDebugUtils.ush"

#include "/Engine/Shared/RaytracingDebugDefinitions.h"
#include "/Engine/Shared/RayTracingDebugTypes.h"

uint OpaqueOnly;

uint VisualizationMode;
uint TriangleHitCountForceNonOpaque;
RaytracingAccelerationStructure TLAS;

RAY_TRACING_ENTRY_RAYGEN(RayTracingDebugHitStatsRGS)
{
	const uint2 PixelPos = DispatchRaysIndex().xy + View.ViewRectMin.xy;
	float2 RenderTargetUV = (float2(PixelPos) + .5f) * View.BufferSizeAndInvSize.zw;

	FRayDesc Ray = CreatePrimaryRay(RenderTargetUV);

	const uint InstanceInclusionMask = (OpaqueOnly ? RAY_TRACING_MASK_OPAQUE : RAY_TRACING_MASK_ALL);
	
	uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES | RAY_FLAG_FORCE_NON_OPAQUE;

	FRayTracingDebugPayload Payload = (FRayTracingDebugPayload)0;
	Payload.SetMiss();

	TraceRay(
		TLAS,
		RayFlags  /*RayFlags*/,
		InstanceInclusionMask,
		0 /*RayContributionToHitGroupIndex*/,
		RAY_TRACING_NUM_SHADER_SLOTS /*MultiplierForGeometryContributionToShaderIndex*/,
		0 /*MissShaderIndex*/,
		Ray.GetNativeDesc(),
		Payload);
}
