// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Shared/RayTracingDebugTypes.h"
#include "../Common.ush"
#include "../Hash.ush"

#include "RayTracingCommon.ush"
#include "RayTracingDebugUtils.ush"
#include "RayTracingHitGroupCommon.ush"

RAY_TRACING_ENTRY_ANY_HIT(RayTracingDebugHitStatsAHS,
	FRayTracingDebugPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	const uint Index = InstanceIndex();

	const uint GPUSceneInstanceId = GetInstanceUserData();
	const FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);

	RayTracingDebugHitStatsUniformBuffer.HitStatsOutput[Index].PrimitiveID = InstanceSceneData.PrimitiveId;
	InterlockedAdd(RayTracingDebugHitStatsUniformBuffer.HitStatsOutput[Index].Count, 1);
}

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingDebugHitStatsCHS,
	FRayTracingDebugPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
}