// Copyright Epic Games, Inc. All Rights Reserved.
#include "../Common.ush"
#include "../SceneData.ush"
#include "../ShaderPrint.ush"

#define INVALID_PRIMITIVEID ~0

uint					SelectedNameInfoCount;
uint					SelectedNameCharacterCount;
StructuredBuffer<uint3> SelectedPrimitiveNameInfos;
Buffer<uint>			SelectedPrimitiveNames;

struct FRayTracingSceneDebugHitStatsNameInfo
{
	uint PrimitiveID;
	uint Count;
	uint Length;
	uint Offset;
	uint Pad0;
	uint Pad1;
};

FRayTracingSceneDebugHitStatsNameInfo UnpackDebugNameInfo(uint3 In)
{
	FRayTracingSceneDebugHitStatsNameInfo Out = (FRayTracingSceneDebugHitStatsNameInfo)0;
	Out.PrimitiveID = In.x;
	Out.Count = In.y;
	Out.Offset = (In.z) & 0xFFFF;
	Out.Length = (In.z >> 16) & 0xFF;

	return Out;
}

// Simple local bin sort based on shared atomics
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void RayTracingSceneDebugHitStatsRenderCS(uint3 DispatchThread : SV_DispatchThreadID)
{
	uint EntryID = DispatchThread.x;
	if (EntryID >= SelectedNameInfoCount)
	{
		return;
	}

	uint StartOffsetX = 10;
	uint StartOffsetY = 30;
	uint SlotHeight = 8;
	if (EntryID == 0 && SelectedNameInfoCount > 0)
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(StartOffsetX, StartOffsetY + SlotHeight));
		Print(Context, TEXT("Rank        Prim.ID     Hit Count    Name"), FontWhite);
	}

	if (SelectedNameInfoCount > 0)
	{
		FRayTracingSceneDebugHitStatsNameInfo NameInfo = UnpackDebugNameInfo(SelectedPrimitiveNameInfos[EntryID]);
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(StartOffsetX, StartOffsetY + 2 * SlotHeight + EntryID * SlotHeight));

		Print(Context, EntryID, FontYellow);
		Print(Context, NameInfo.PrimitiveID, FontWhite);
		Print(Context, NameInfo.Count, FontWhite);

		for (uint It = 0; It < NameInfo.Length; ++It)
		{
			const uint Char = SelectedPrimitiveNames[It + NameInfo.Offset];
			PrintSymbol(Context, Char, FontWhite);
		}
	}
}
