// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShadingCommon.ush"
#include "../ColorMap.ush"
#include "../Hash.ush"
#include "../Visualization.ush"
#include "../SceneData.ush"
#include "/Engine/Shared/RayTracingDebugTypes.h"

#define MAX_FLT (+3.402823466e+38f)
#define MIN_FLT (-3.402823466e+38f)
#define INVALID_INSTANCE_GPUSCENE_INDEX (0xFFFFFFFF)

float BoundingBoxExtentScale;

StructuredBuffer<FRayTracingInstanceExtraData> InstanceExtraDataBuffer;

struct FInstanceBounds
{
	float3 Center;
	float3 Extent;
};

FInstanceBounds GetTranslatedWorldAABBForInstance(uint GPUSceneInstanceId)
{
	FInstanceSceneData InstanceSceneData = GetInstanceSceneData(GPUSceneInstanceId);

	const FDFVector3 PreViewTranslation = MakeDFVector3(View.TLASPreViewTranslationHigh, View.TLASPreViewTranslationLow);
	const float4x4 LocalToTranslatedWorld = DFFastToTranslatedWorld(InstanceSceneData.LocalToWorld, PreViewTranslation);	

	const float3 BoundingBoxCenter = InstanceSceneData.LocalBoundsCenter.xyz;
	const float3 BoundingBoxExtent = InstanceSceneData.LocalBoundsExtent.xyz;

	float3 TranslatedWorldPositionMin = MAX_FLT;
	float3 TranslatedWorldPositionMax = MIN_FLT;

	for (int x = 0; x < 2; x++)
	{
		for (int y = 0; y < 2; y++)
		{
			for (int z = 0; z < 2; z++)
			{
				float3 Pos;
				Pos.x = x ? -1.0f : 1.0f;
				Pos.y = y ? -1.0f : 1.0f;
				Pos.z = z ? -1.0f : 1.0f;

				const float3 LocalPosition = Pos.xyz * BoundingBoxExtent + BoundingBoxCenter;

				const float3 TranslatedWorldPosition = mul(float4(LocalPosition, 1.0), LocalToTranslatedWorld).xyz;

				TranslatedWorldPositionMin = min(TranslatedWorldPosition, TranslatedWorldPositionMin);
				TranslatedWorldPositionMax = max(TranslatedWorldPosition, TranslatedWorldPositionMax);
			}
		}
	}		
	
	FInstanceBounds InstanceBounds;
	InstanceBounds.Center = (TranslatedWorldPositionMax + TranslatedWorldPositionMin) * 0.5;
	InstanceBounds.Extent = (TranslatedWorldPositionMax - TranslatedWorldPositionMin) * 0.5;

	return InstanceBounds;
}

void InstanceOverlapMainVS(
	float4 InPosition : ATTRIBUTE0,
	uint InstanceIndex : SV_InstanceID,	
	out float4 OutPosition : SV_POSITION
)
{
	const FRayTracingInstanceExtraData InstanceExtraData = InstanceExtraDataBuffer[InstanceIndex];

	if (InstanceExtraData.GPUSceneInstanceId != INVALID_INSTANCE_GPUSCENE_INDEX)
	{
		const FInstanceBounds InstanceBounds = GetTranslatedWorldAABBForInstance(InstanceExtraData.GPUSceneInstanceId);

		// Make sure extent is non-zero to avoid flickering.
		const float3 Extent = max(InstanceBounds.Extent, BoundingBoxExtentScale);
		const float3 TranslatedWorldPosition = InPosition.xyz * Extent * BoundingBoxExtentScale + InstanceBounds.Center;

		OutPosition = mul(float4(TranslatedWorldPosition, 1.0), PrimaryView.TranslatedWorldToClip);
	}
	else
	{
		OutPosition = float4(0, 0, 0, -1);
	}
}

void InstanceOverlapMainPS(
	in float4 Position : SV_POSITION, 
	out float4 OutColor : SV_Target0
)
{
	OutColor = 1.0;
}

Texture2D<float> InputDepth;

void ConvertToDeviceDepthPS(
	in  float4 SvPosition : SV_POSITION,
	out float  OutDepth : SV_DEPTH)
{
	OutDepth = InputDepth.Load(int3(SvPosition.xy, 0)).r;
}

float HeatmapScale;
Texture2D<float> InstanceOverlap;

void BlendInstanceOverlapPS(
	in float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
)
{
	const uint2 PixelCoord = uint2(SvPosition.xy);

	const float3 HeatmapColor = InstanceOverlap[PixelCoord] == 0.0 ? 0.0 : ColorMapTurbo(InstanceOverlap[PixelCoord] / HeatmapScale);
	const float BlendValue = 0.5;
	
	OutColor = float4(HeatmapColor, BlendValue);
}