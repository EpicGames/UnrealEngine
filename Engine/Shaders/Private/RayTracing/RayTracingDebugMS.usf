// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../Hash.ush"

#include "RayTracingCommon.ush"
#include "RayTracingDebugUtils.ush"
#include "RayTracingHitGroupCommon.ush"

RAY_TRACING_ENTRY_MISS(RayTracingDebugMS, FRayTracingDebugPayload, Payload)
{
	Payload.SetMiss();
}
