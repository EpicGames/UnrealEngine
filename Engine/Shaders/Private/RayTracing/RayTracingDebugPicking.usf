// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "RayTracingDebugUtils.ush"

#include "/Engine/Shared/RayTracingDebugTypes.h"

uint OpaqueOnly;

RaytracingAccelerationStructure TLAS;

StructuredBuffer<FRayTracingInstanceExtraData> InstancesExtraData;
StructuredBuffer<FRayTracingInstanceDebugData> InstancesDebugData;
StructuredBuffer<FRayTracingPickingFeedback> PickingBuffer;

RWStructuredBuffer<FRayTracingPickingFeedback> PickingOutput;
StructuredBuffer<FPlatformRayTracingInstanceDescriptor> InstanceBuffer;

RAY_TRACING_ENTRY_RAYGEN_OR_INLINE(RayTracingDebugPicking)
{
	const uint2 PixelPos = (float2(View.CursorPosition) * View.ViewResolutionFraction);
	float2 RenderTargetUV = (float2(PixelPos) + .5f) * View.BufferSizeAndInvSize.zw;

	FRayDesc Ray = CreatePrimaryRay(RenderTargetUV);

	const uint InstanceInclusionMask = (OpaqueOnly ? RAY_TRACING_MASK_OPAQUE : RAY_TRACING_MASK_ALL);

	FRayTracingDebugPayload Payload = (FRayTracingDebugPayload)0;
	Payload.SetMiss();

#if COMPUTESHADER
	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();

	FTraceRayInlineResult TraceRayResult = TraceRayInline(
		TLAS,
		RAY_FLAG_CULL_BACK_FACING_TRIANGLES,
		InstanceInclusionMask,
		Ray.GetNativeDesc(),
		TraceRayInlineContext
	);
	
	Payload.HitT = TraceRayResult.HitT;

	Payload.TriangleIndex = TraceRayResult.PrimitiveIndex;
	Payload.InstanceIndex = TraceRayResult.InstanceIndex;
	Payload.GeometryIndex = TraceRayResult.GeometryIndex;
#else
	TraceRay(
		TLAS,
		RAY_FLAG_CULL_BACK_FACING_TRIANGLES  /*RayFlags*/,
		InstanceInclusionMask,
		0 /*RayContributionToHitGroupIndex*/,
		RAY_TRACING_NUM_SHADER_SLOTS /*MultiplierForGeometryContributionToShaderIndex*/,
		0 /*MissShaderIndex*/,
		Ray.GetNativeDesc(),
		Payload);
#endif

	FRayTracingPickingFeedback PickingFeedback = (FRayTracingPickingFeedback)0xFFFFFFFFu;

	if (Payload.IsHit())
	{
		const FRayTracingInstanceExtraData InstanceExtraData = InstancesExtraData[Payload.InstanceIndex];
		const FRayTracingInstanceDebugData InstanceDebugData = InstancesDebugData[InstanceExtraData.SceneInstanceIndex];
		const FPlatformRayTracingInstanceDescriptor InstanceDescriptor = InstanceBuffer[Payload.InstanceIndex];

		PickingFeedback.GeometryAddress = InstanceDebugData.GeometryAddress;

		PickingFeedback.InstanceIndex = Payload.InstanceIndex;
		PickingFeedback.TriangleIndex = Payload.TriangleIndex;
		PickingFeedback.GeometryIndex = Payload.GeometryIndex;

		PickingFeedback.InstanceId = GetRayTracingInstanceDescriptorInstanceId(InstanceDescriptor);
		PickingFeedback.Flags = TranslateRayTracingPlatformInstanceFlags(GetRayTracingInstanceDescriptorFlags(InstanceDescriptor));
		PickingFeedback.Mask = GetRayTracingInstanceDescriptorMask(InstanceDescriptor);
		PickingFeedback.InstanceContributionToHitGroupIndex = GetRayTracingInstanceDescriptorInstanceContributionToHitGroupIndex(InstanceDescriptor);
	}

	PickingOutput[0] = PickingFeedback;
}
