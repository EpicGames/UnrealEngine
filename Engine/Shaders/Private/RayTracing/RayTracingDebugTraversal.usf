// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShadingCommon.ush"
#include "../ColorMap.ush"
#include "../BlueNoise.ush"

#include "RayTracingCommon.ush"
#include "RayTracingDeferredMaterials.ush"

#include "RayTracingDeferredShadingCommon.ush"
#include "../Substrate/Substrate.ush"
#include "../Lumen/LumenPosition.ush"

#include "/Engine/Shared/RaytracingDebugDefinitions.h"

#include "TraceRayInline.ush"

RWTexture2D<float4> Output;
RaytracingAccelerationStructure TLAS;

uint VisualizationMode;
float TraversalBoxScale;
float TraversalClusterScale;
float TraversalTriangleScale;

float NormalBias;
float TraceDistance;

struct FNaniteTraceRayInlineCallback : FDefaultTraceRayInlineCallback
{
	uint ClusterIntersectionCount;

	bool OnProceduralPrimitive(float3 ObjectRayOrigin, float3 ObjectRayDirection, float RayTMin, inout float RayTCurrent, FTraceRayInlineProceduralIntersectionParameters ProceduralIntersectionParameters)
	{
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
		uint PrimitiveID = ProceduralIntersectionParameters.InstanceID;
		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveID);
		uint PrimitiveInstanceIndex = 0; // TODO
		uint InstanceId = PrimitiveData.InstanceSceneDataOffset + PrimitiveInstanceIndex;
		FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceId);

		// TODO: implement support for intersection with procedural primitives
#endif
		return false;
	}
};

FRayDesc CreateSecondaryRay(uint2 PixelCoord, float2 ScreenUV, out bool bIsValid)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));
	bIsValid = IsSubstrateMaterial(TopLayerData);
	const float3 WorldNormal = TopLayerData.WorldNormal;
	const float Depth = ConvertFromDeviceZ(SceneDepthTexture.Load(int3(PixelCoord, 0)).r);
#else
	const FGBufferData GBufferData = GetGBufferDataFromSceneTexturesLoad(PixelCoord);
	bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
	const float3 WorldNormal = GBufferData.WorldNormal;
	const float Depth = GBufferData.Depth;
#endif

	FRayDesc Ray = (FRayDesc)0;

	if (bIsValid)
	{
		float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromScreenUV(ScreenUV, Depth);
		float3x3 TangentBasis = GetTangentBasis(WorldNormal);

		float2 UniformRandom = BlueNoiseVec2(PixelCoord, View.StateFrameIndex);

		float4 HemisphereSample = CosineSampleHemisphere(UniformRandom);
		float3 RayDirection = mul(HemisphereSample.xyz, TangentBasis);
		
		Ray.Origin = TranslatedWorldPosition;
		Ray.TMin = 0;
		Ray.Direction = RayDirection;
		Ray.TMax = TraceDistance;

		ApplyCameraRelativeDepthBias(Ray, PixelCoord, ConvertToDeviceZ(Depth), WorldNormal, NormalBias);
	}

	return Ray;
}

[numthreads(INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X, INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y, 1)]
void RayTracingDebugTraversalCS(uint3 DispatchThread : SV_DispatchThreadID)
{
	uint2 PixelCoord = DispatchThread.xy + View.ViewRectMin.xy;

	float2 RenderTargetUV = (float2(PixelCoord) + .5f) * View.BufferSizeAndInvSize.zw;

	FRayDesc Ray = CreatePrimaryRay(RenderTargetUV);
	bool bIsValid = true;

	if (VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL
		|| VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER
		|| VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE
		|| VisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE)
	{
		Ray = CreateSecondaryRay(PixelCoord, RenderTargetUV, bIsValid);
	}

	if (!bIsValid)
	{
		Output[PixelCoord] = 0.0f;
		return;
	}

	uint RayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES | RAY_FLAG_FORCE_OPAQUE;

	FNaniteTraceRayInlineCallback NaniteCallback = (FNaniteTraceRayInlineCallback)0;

	FTraceRayInlineContext TraceRayInlineContext = CreateTraceRayInlineContext();
	TraceRayInlineContext.bProcedural = ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE;

	FTraceRayInlineResult TraceResult = TraceRayInlineWithCallback(TLAS, RayFlags, RAY_TRACING_MASK_ALL, Ray.GetNativeDesc(), TraceRayInlineContext, NaniteCallback);

	float4 Result = 0;

	const uint Intersections = TraceResult.TraversalStatistics.NodeIntersectionCount + NaniteCallback.ClusterIntersectionCount + TraceResult.TraversalStatistics.TriangleIntersectionCount;
	switch (VisualizationMode)
	{
	default:
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_ALL:
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL:
		Result.rgb = ColorMapTurbo(Intersections / (TraversalBoxScale + TraversalTriangleScale));
		break;
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_CLUSTER:
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER:
		Result.rgb = ColorMapTurbo(NaniteCallback.ClusterIntersectionCount / TraversalClusterScale);
		break;
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_NODE:
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE:
		Result.rgb = ColorMapTurbo(TraceResult.TraversalStatistics.NodeIntersectionCount / TraversalBoxScale);
		break;
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_TRIANGLE:
	case RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE:
		Result.rgb = ColorMapTurbo(TraceResult.TraversalStatistics.TriangleIntersectionCount / TraversalTriangleScale);
		break;
	}

#if PRINT_TRAVERSAL_STATISTICS
	TraceRayInlineAccumulateStatistics();
#endif

	Output[PixelCoord] = Result;
}
