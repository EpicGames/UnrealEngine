// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/RayTracingPayloadType.h"
#include "RayTracingCommon.ush"

struct FRayTracingDebugPayload : FMinimalPayload
{
	uint GPUSceneInstanceId;
	uint TriangleIndex;
	uint InstanceIndex;
	uint GeometryIndex;
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	uint ClusterId;
#endif

	float3 WorldNormal;
	
	uint TriangleHitCountPerRay;
	uint Flags;
	
	void SetFrontFace()
	{
		Flags |= RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE;
	}
	bool IsFrontFace()
	{
		return (Flags & RAY_TRACING_PAYLOAD_OUTPUT_FLAG_FRONT_FACE) != 0;
	}
};
#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_RAYTRACING_DEBUG)
CHECK_RT_PAYLOAD_SIZE(FRayTracingDebugPayload)
#endif
