// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef USE_RAY_TRACING_DECAL_GRID
#define USE_RAY_TRACING_DECAL_GRID 0
#endif

#include "../Common.ush"
#include "../Visualization.ush"
#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/PathTracingDefinitions.h"

struct FDecalLoopCount {
	uint NumDecals;
#if USE_RAY_TRACING_DECAL_GRID
	uint DecalGridOffset;
#endif
};

#if USE_RAY_TRACING_DECAL_GRID

FDecalLoopCount DecalGridLookup(float3 TranslatedWorldPos)
{
	FDecalLoopCount Result;
	Result.NumDecals = 0;
	Result.DecalGridOffset = ~0u;

	if (all(RayTracingDecals.TranslatedBoundMin <= TranslatedWorldPos) && all(TranslatedWorldPos <= RayTracingDecals.TranslatedBoundMax))
	{
		float3 P = TranslatedWorldPos - RayTracingDecals.TranslatedBoundMin;
		float3 D = RayTracingDecals.TranslatedBoundMax - RayTracingDecals.TranslatedBoundMin;

		int2 UVx = int2(floor(RayTracingDecals.GridResolution * P.yz / D.yz));
		int2 UVy = int2(floor(RayTracingDecals.GridResolution * P.xz / D.xz));
		int2 UVz = int2(floor(RayTracingDecals.GridResolution * P.xy / D.xy));

		uint DecalNx = RayTracingDecals.Grid.Load(int4(UVx, 0, 0));
		uint DecalNy = RayTracingDecals.Grid.Load(int4(UVy, 1, 0));
		uint DecalNz = RayTracingDecals.Grid.Load(int4(UVz, 2, 0));
		uint DecalN = DecalNx;
		uint DecalOffset = RayTracingDecals.GridMaxCount * (0 + 3 * (UVx.x + UVx.y * RayTracingDecals.GridResolution));
		if ((RayTracingDecals.GridAxis > 2 && DecalNy < DecalN) || RayTracingDecals.GridAxis == 1)
		{
			DecalN = DecalNy;
			DecalOffset = RayTracingDecals.GridMaxCount * (1 + 3 * (UVy.x + UVy.y * RayTracingDecals.GridResolution));
		}
		if ((RayTracingDecals.GridAxis > 2 && DecalNz < DecalN) || RayTracingDecals.GridAxis == 2)
		{
			DecalN = DecalNz;
			DecalOffset = RayTracingDecals.GridMaxCount * (2 + 3 * (UVz.x + UVz.y * RayTracingDecals.GridResolution));
		}

		Result.DecalGridOffset = DecalOffset;
		Result.NumDecals += DecalN;
	}
	return Result;
}

float3 DecalGridVisualize(FDecalLoopCount LoopCount, int DebugMode, uint MaxCount)
{
	const float3 OutsideColor = 0.18;
	const float3 EmptyColor = 0.36;

	if (LoopCount.DecalGridOffset != ~0u)
	{
		switch (DebugMode)
		{
            case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_LIST_HASH:
            {
                // color by unique decal list
                uint H = 0;
                for (int Index = 0, Num = LoopCount.NumDecals; Index < Num; Index++)
                {
                    H = MurmurMix(H + RayTracingDecals.GridData[LoopCount.DecalGridOffset + Index]);
                }
                if (LoopCount.NumDecals < 1)
                {
                    return EmptyColor;
                }
                return HUEtoRGB((H & 0xFFFFFF) * 5.96046447754e-08);
            }
            case PATH_TRACER_DEBUG_VIZ_DECAL_GRID_AXIS:
            {
                // color by chosen axis
                int Axis = (LoopCount.DecalGridOffset / RayTracingDecals.GridMaxCount) % 3;
                if (Axis == 0) return float3(0.9, 0.2, 0.2);
                if (Axis == 1) return float3(0.2, 0.9, 0.2);
                if (Axis == 2) return float3(0.2, 0.2, 0.9);
                return float3(0.8, 0.2, 0.8); // error
            }
            default:
            {
                // default mode - color by decal count
                uint N = LoopCount.NumDecals;
                if (N < 1)
                {
                    return EmptyColor;
                }
                if (N > MaxCount)
                {
                    return float3(1, 0.1, 1);
                }
                float t = saturate(float(N) / MaxCount);
                return BlueGreenRedColorMap(t);
            }
		}
	}
	// outside decal grid bounds
	return OutsideColor;
}


uint GetDecalId(uint Index, FDecalLoopCount LoopCount)
{
	return RayTracingDecals.GridData[LoopCount.DecalGridOffset + Index];
}

#else

FDecalLoopCount DecalGridLookup(float3 TranslatedWorldPos)
{
	uint NumDecals = min(RayTracingDecals.Count, RAY_TRACING_DECAL_COUNT_MAXIMUM);
	FDecalLoopCount Result;
	Result.NumDecals = NumDecals;
	return Result;
}

float3 DecalGridVisualize(FDecalLoopCount LoopCount, int DebugMode, uint MaxCount)
{
	return 0.5;
}

uint GetDecalId(uint Index, FDecalLoopCount LoopCount)
{
	return Index;
}

#endif
