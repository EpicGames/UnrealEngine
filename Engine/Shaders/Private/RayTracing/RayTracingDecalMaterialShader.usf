// Copyright Epic Games, Inc. All Rights Reserved.

#define PATH_TRACING 1
#define SCENE_TEXTURES_DISABLED 1 // Ray tracing shaders cannot access scene textures

// This should be good enough for ray tracing and avoids having to bind an extra buffer
#define EYE_ADAPTATION_PREV_FRAME_EXPOSURE 1

#define SUBSTRATE_INLINE_SHADING 1

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/PathTracing/PathTracingShaderUtils.ush"

#define DecalPositionHigh RayTracingDecalParameters.DecalPositionHigh
#define WorldToDecal RayTracingDecalParameters.WorldToDecal
#define SvPositionToDecal WorldToDecal
#define DecalToWorld RayTracingDecalParameters.DecalToWorld
#define DecalToWorldInvScale RayTracingDecalParameters.DecalToWorldInvScale
#define DecalOrientation RayTracingDecalParameters.DecalOrientation
#define DecalParams RayTracingDecalParameters.DecalParams
#define DecalColorParam RayTracingDecalParameters.DecalColorParam
#define DecalWriteFlags RayTracingDecalParameters.DecalWriteFlags

#include "/Engine/Generated/Material.ush"


struct FDecalData
{
	float Opacity;
	float3 BaseColor;
	float3 WorldNormal;
	float3 MetallicSpecularRoughness;
	float3 Emissive;
	
	void CombineWithPayload(inout FDecalShaderPayload Payload, uint WriteFlags, float OpacityMult)
	{
		Opacity *= OpacityMult;
		float Transparency = 1.0 - Opacity;
		if (WriteFlags & DECAL_WRITE_BASE_COLOR_FLAG)
		{
			float4 DestBaseColor = Payload.GetBaseColor();
			DestBaseColor.rgb = BaseColor * Opacity + DestBaseColor.rgb * Transparency;
			DestBaseColor.a *= Transparency;
			Payload.SetBaseColor(DestBaseColor);
		}
		if (WriteFlags & DECAL_WRITE_NORMAL_FLAG)
		{
			float4 DestWorldNormal = Payload.GetWorldNormal();
			DestWorldNormal.xyz = WorldNormal * Opacity + DestWorldNormal.xyz * Transparency;
			DestWorldNormal.w *= Transparency;
			Payload.SetWorldNormal(DestWorldNormal);
		}
		if (WriteFlags & DECAL_WRITE_ROUGHNESS_SPECULAR_METALLIC_FLAG)
		{
			float4 DestMetallicSpecularRoughness = Payload.GetMetallicSpecularRoughness();
			DestMetallicSpecularRoughness.rgb = MetallicSpecularRoughness * Opacity + DestMetallicSpecularRoughness.rgb * Transparency;
			DestMetallicSpecularRoughness.a *= Transparency;
			Payload.SetMetallicSpecularRoughness(DestMetallicSpecularRoughness);
		}
		if (WriteFlags & DECAL_WRITE_EMISSIVE_FLAG)
		{
			Payload.SetEmissive(Payload.GetEmissive() + Emissive * Opacity);
		}
	}
};



FDecalData GetDecalData(FPixelMaterialInputs PixelMaterialInputs, FMaterialPixelParameters MaterialParameters)
{
	FDecalData Result = (FDecalData)0;
#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	FSubstratePixelHeader SubstrateHeader = MaterialParameters.GetFrontSubstrateHeader();

	if (SubstrateHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;

		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*bRoughDiffuseEnabled*/, 0, false/*bRoughnessTracking*/);
		SubstrateHeader.SubstrateUpdateTree(V, Settings);

		// Since the Convert-To-Decal node forces parameter blending for the entire Substrate tree, the BSDF we are interested in will be in 0.
		FSubstrateDBuffer DBuffer = SubstrateHeader.SubstrateConvertToDBuffer(SubstrateHeader.SubstrateTree.BSDFs[0]);
		Result.Opacity = DBuffer.Coverage;
		Result.BaseColor = DBuffer.BaseColor;
		Result.WorldNormal = DBuffer.WorldNormal;
		Result.MetallicSpecularRoughness = float3(DBuffer.Metallic, DBuffer.Specular, DBuffer.Roughness);
		Result.Emissive = DBuffer.Emissive;
	}
#else // SUBSTRATE_ENABLED
	Result.Opacity = GetMaterialOpacity(PixelMaterialInputs);
	Result.BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	Result.WorldNormal = MaterialParameters.WorldNormal;
	Result.MetallicSpecularRoughness = float3(GetMaterialMetallic(PixelMaterialInputs), GetMaterialSpecular(PixelMaterialInputs), GetMaterialRoughness(PixelMaterialInputs));
	Result.Emissive = GetMaterialEmissive(PixelMaterialInputs);
#endif // SUBSTRATE_ENABLED
	return Result;
}


#if RAYCALLABLESHADER
RAY_TRACING_ENTRY_CALLABLE(RayTracingDecalMaterialShader, FDecalShaderPayload, Payload)
{
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = Payload.TranslatedWorldPos;

	float4 DecalPos = mul(float4(TranslatedWorldPosition.xyz, 1), WorldToDecal);

	if (any(abs(DecalPos.xyz) > 1))
	{
		// out of bounds, nothing to do
		return;
	}

	float3 CameraVector = normalize(PrimaryView.TranslatedWorldCameraOrigin - TranslatedWorldPosition.xyz);

	// can be optimized
	float3 DecalVector = DecalPos.xyz * 0.5f + 0.5f;

	// Swizzle so that DecalVector.xy are perpendicular to the projection direction and DecalVector.z is distance along the projection direction
	float3 SwizzlePos = DecalVector.zyx;

	// By default, map textures using the vectors perpendicular to the projection direction
	float2 DecalUVs = SwizzlePos.xy;

	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
#if NUM_MATERIAL_TEXCOORDS
	for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
	{
		MaterialParameters.TexCoords[CoordinateIndex] = DecalUVs;
	}
#endif
	MaterialParameters.TwoSidedSign = 1;
	MaterialParameters.VertexColor = 1;
	MaterialParameters.CameraVector = CameraVector;
	MaterialParameters.SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);
	MaterialParameters.ScreenPosition = SvPositionToResolvedScreenPosition(MaterialParameters.SvPosition);
	MaterialParameters.LightVector = SwizzlePos;

	MaterialParameters.AbsoluteWorldPosition = MaterialParameters.WorldPosition_NoOffsets = DFFastSubtract(TranslatedWorldPosition.xyz, PrimaryView.PreViewTranslation);
	MaterialParameters.WorldPosition_CamRelative = MaterialParameters.WorldPosition_NoOffsets_CamRelative = TranslatedWorldPosition.xyz;
	MaterialParameters.LWCData = MakeMaterialLWCData(MaterialParameters);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

	const float DecalFading = saturate(4 - 4 * abs(SwizzlePos.z * 2 - 1)) * DecalParams.x;

	FDecalData Data = GetDecalData(PixelMaterialInputs, MaterialParameters);

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	{
		// Virtual texturing feedback logic
		// NOTE: decals are only evaluated for camera rays (and sharp reflections)
		FinalizeVirtualTextureFeedback(
			MaterialParameters.VirtualTextureFeedback,
			MaterialParameters.SvPosition,
			View.VTFeedbackBuffer
		);
	}
#endif
	
	Data.CombineWithPayload(Payload, DecalWriteFlags, DecalFading);

}
#endif // RAYCALLABLESHADER

#if RAYHITGROUPSHADER

#ifdef NEEDS_VERTEX_FACTORY_INTERPOLATION
#undef NEEDS_VERTEX_FACTORY_INTERPOLATION
#endif
// Needed for VertexFactoryInterpolate to interpolate attributes from vertices to hit point
#define NEEDS_VERTEX_FACTORY_INTERPOLATION 1

#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

#include "/Engine/Generated/VertexFactory.ush"

#include "/Engine/Private/RayTracing/RayTracingCalcInterpolants.ush"

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingDecalMaterialCHS,
	FDecalShaderPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.HitT = RayTCurrent();

	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + WorldRayDirection() * RayTCurrent();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	// this is a newer codepath that is both more flexible and allows for more direct calculation compared to the other codepath
	// TODO: implement such a method for all vertex factories
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), GetClusterID(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FVertexFactoryInterpolantsVSToPS Interpolants;
	float3 GeoNormal = 0;
	CalcInterpolants((FRayCone)0, Attributes, Interpolants, GeoNormal);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif
	FPixelMaterialInputs PixelMaterialInputs;

	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		const bool bIsFrontFace = HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE;

		MaterialParameters.CameraVector = -WorldRayDirection();

		// #dxr_todo: UE-72130 support world position offset 
		// #if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
		// CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, BasePassInterpolants.PixelPositionExcludingWPO);
		// #else
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
		// #endif
	}

	FDecalData Data = GetDecalData(PixelMaterialInputs, MaterialParameters);
	
#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	{
		// Virtual texturing feedback logic
		// NOTE: decals are only evaluated for camera rays (and sharp reflections)
		FinalizeVirtualTextureFeedback(
			MaterialParameters.VirtualTextureFeedback,
			MaterialParameters.SvPosition,
			View.VTFeedbackBuffer
		);
	}
#endif

#if DECAL_USE_REVERSE_CULLING
	Data.WorldNormal = -Data.WorldNormal; // flip normal (see matching logic in the PathTracer)
#endif

	Data.CombineWithPayload(Payload, DECAL_PAYLOAD_FLAGS, 1.0);
}

#if USE_MATERIAL_ANY_HIT_SHADER

RAY_TRACING_ENTRY_ANY_HIT(RayTracingDecalMaterialAHS,
	FDecalShaderPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + WorldRayDirection() * RayTCurrent();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

	FVertexFactoryInterpolantsVSToPS Interpolants;

	CalcInterpolants((FRayCone)0, Attributes, Interpolants);

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;

	const bool bIsFrontFace = HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE;

	{
		const float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);

		MaterialParameters.CameraVector = -WorldRayDirection();

		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
	}

	const float Opacity = GetDecalData(PixelMaterialInputs, MaterialParameters).Opacity;
	if (Opacity <= 0.00001f)
	{
		IgnoreHit();
		return;
	}
}

#endif // USE_MATERIAL_ANY_HIT_SHADER

#endif // RAYHITGROUPSHADER
