// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/RayTracing/RayTracingCommon.ush"

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingDefaultOpaqueDecalMaterialCHS,
	FDecalShaderPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.HitT = RayTCurrent();
	// Equivalent to a pure black BaseColor only decal
	Payload.SetBaseColor(float4(0, 0, 0, 0));
}

RAY_TRACING_ENTRY_CLOSEST_HIT(RayTracingDefaultHiddenDecalMaterialCHS,
	FDecalShaderPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	// nothing to do here since AHS will ignore the hit
}

RAY_TRACING_ENTRY_ANY_HIT(RayTracingDefaultHiddenDecalMaterialAHS,
	FDecalShaderPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	IgnoreHit();
}
