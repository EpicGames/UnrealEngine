// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "RayTracingCommon.ush"
#include "RayTracingHitGroupCommon.ush"

RAY_TRACING_ENTRY_CLOSEST_HIT(DeferredMaterialCHS,
	FDeferredMaterialPayload, Payload,
	FRayTracingIntersectionAttributes, Attributes)
{
	Payload.SortKey = GetHitGroupUserData();
	Payload.HitT    = RayTCurrent();
}

RAY_TRACING_ENTRY_MISS(DeferredMaterialMS, 
	FDeferredMaterialPayload, Payload)
{
	Payload.SetMiss();
}

