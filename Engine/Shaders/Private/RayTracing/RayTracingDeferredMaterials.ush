// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RayTracingCommon.ush"

#ifdef OVERRIDE_RAYTRACINGDEFERREDMATERIALS_USH
#include "/Platform/Private/RayTracingDeferredMaterials.ush"
#else // OVERRIDE_RAYTRACINGDEFERREDMATERIALS_USH

struct FRayIntersectionBookmark
{
	uint Data[2];
};

#if DIM_AMD_HIT_TOKEN

#ifndef AGS_RAY_HIT_TOKEN
#define AGS_RAY_HIT_TOKEN 1
#endif

#include "/Engine/Shared/ThirdParty/AMD/ags_shader_intrinsics_dx12.h"

#define TraceDeferredMaterialGatherRay(TLAS, RayFlags, InstanceInclusionMask, Ray, OutBookmark, InOutPayload){\
	TraceRay(TLAS, RayFlags, InstanceInclusionMask, RAY_TRACING_SHADER_SLOT_MATERIAL, RAY_TRACING_NUM_SHADER_SLOTS, 0, Ray, InOutPayload);\
	OutBookmark.Data[0] = AmdGetLastHitToken().x;\
	OutBookmark.Data[1] = AmdGetLastHitToken().y;\
}

#define TraceDeferredMaterialShadingRay(TLAS, RayFlags, InstanceInclusionMask, Ray, Bookmark, InOutPayload){\
	AmdSetHitToken(uint2(Bookmark.Data[0], Bookmark.Data[1]));\
	TraceRay(TLAS, RayFlags, InstanceInclusionMask, RAY_TRACING_SHADER_SLOT_MATERIAL, RAY_TRACING_NUM_SHADER_SLOTS, 0, Ray, InOutPayload);\
}

#else // DIM_AMD_HIT_TOKEN

#define TraceDeferredMaterialGatherRay(TLAS, RayFlags, InstanceInclusionMask, Ray, OutBookmark, InOutPayload){\
	TraceRay(TLAS, RayFlags, InstanceInclusionMask, RAY_TRACING_SHADER_SLOT_MATERIAL, RAY_TRACING_NUM_SHADER_SLOTS, 0, Ray, InOutPayload);\
	OutBookmark.Data[0] = asuint(InOutPayload.HitT);\
	OutBookmark.Data[1] = asuint(InOutPayload.HitT);\
}

#define TraceDeferredMaterialShadingRay(TLAS, RayFlags, InstanceInclusionMask, Ray, Bookmark, InOutPayload){\
	RayDesc BookmarkRay = Ray;\
	BookmarkRay.TMin = asfloat(Bookmark.Data[0]) - 0.5f;\
	BookmarkRay.TMax = asfloat(Bookmark.Data[1]) + 0.5f;\
	TraceRay(TLAS, RayFlags, InstanceInclusionMask, RAY_TRACING_SHADER_SLOT_MATERIAL, RAY_TRACING_NUM_SHADER_SLOTS, 0, BookmarkRay, InOutPayload);\
}

#endif // DIM_AMD_HIT_TOKEN

#endif // OVERRIDE_RAYTRACINGDEFERREDMATERIALS_USH
