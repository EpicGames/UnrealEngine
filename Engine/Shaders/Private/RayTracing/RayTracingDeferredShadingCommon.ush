// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../DeferredShadingCommon.ush"

//#dxr-todo: workaround for flickering. UE-87281

FGBufferData GetGBufferDataFromSceneTexturesLoad(uint2 PixelCoord, bool bGetNormalizedNormal = true)
{
	float4 GBufferA = GBufferATexture.Load(int3(PixelCoord, 0));
	float4 GBufferB = GBufferBTexture.Load(int3(PixelCoord, 0));
	float4 GBufferC = GBufferCTexture.Load(int3(PixelCoord, 0));
	float4 GBufferD = GBufferDTexture.Load(int3(PixelCoord, 0));
	float4 GBufferE = GBufferETexture.Load(int3(PixelCoord, 0));
	float4 GBufferF = GBufferFTexture.Load(int3(PixelCoord, 0));

#if GBUFFER_HAS_VELOCITY
	float4 GBufferVelocity = GBufferVelocityTexture.Load(int3(PixelCoord, 0));
#else
	float4 GBufferVelocity = 0.0f;
#endif

	uint CustomStencil = 0;
	float CustomNativeDepth = 0;

	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;;

	float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	return DecodeGBufferData(GBufferA, GBufferB, GBufferC, GBufferD, GBufferE, GBufferF, GBufferVelocity, CustomNativeDepth, CustomStencil, SceneDepth, bGetNormalizedNormal, CheckerFromPixelPos(PixelCoord));
}

FGBufferData GetGBufferDataFromPayload(in FMaterialClosestHitPayload Payload)
{
	FGBufferData GBufferData = (FGBufferData)0;
#if !SUBSTRATE_ENABLED || (SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==0)
	GBufferData.Depth = 1.f; // Do not use depth
	GBufferData.WorldNormal = Payload.WorldNormal;
	GBufferData.BaseColor = Payload.BaseColor;
	GBufferData.Metallic = Payload.Metallic;
	GBufferData.Specular = Payload.Specular;
	GBufferData.Roughness = Payload.Roughness;
	GBufferData.CustomData = Payload.CustomData;
	GBufferData.GBufferAO = Payload.GBufferAO;
	GBufferData.IndirectIrradiance = (Payload.IndirectIrradiance.x + Payload.IndirectIrradiance.y + Payload.IndirectIrradiance.z) / 3.f;
	GBufferData.ShadingModelID = Payload.ShadingModelID;
	GBufferData.SpecularColor = Payload.SpecularColor;
	GBufferData.DiffuseColor = Payload.DiffuseColor;
	GBufferData.WorldTangent = Payload.WorldTangent;
	GBufferData.Anisotropy = Payload.Anisotropy;
#endif
	return GBufferData;
}

FGBufferData GetGBufferDataFromPayload(in FPackedMaterialClosestHitPayload Payload)
{
	FRayDesc DummyRay = (FRayDesc)0; // Not needed to unpack GBuffer
	return GetGBufferDataFromPayload(UnpackRayTracingPayload(Payload, DummyRay));
}