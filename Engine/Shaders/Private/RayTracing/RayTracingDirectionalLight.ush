// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "../MonteCarlo.ush"
#include "../LightData.ush"
#include "RayTracingCommon.ush"

void GenerateDirectionalLightOcclusionRay(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax)
{
	const float SinThetaMax = LightParameters.SourceRadius;
	const float4 DirAndPdf = UniformSampleConeRobust(RandSample, SinThetaMax * SinThetaMax);
	RayOrigin = TranslatedWorldPosition;
	RayDirection = TangentToWorld(DirAndPdf.xyz, LightParameters.Direction);
	RayTMin = 0.0;
	RayTMax = 1.0e27;
}
