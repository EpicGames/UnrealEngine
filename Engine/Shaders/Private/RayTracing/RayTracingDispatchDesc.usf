// Copyright Epic Games, Inc. All Rights Reserved.

/*==========================================================================
	RayTracingDispatchDesc.usf: Generate indirect ray tracing dispatch args 
	using header from CPU (uploaded as constants) and dimensions from GPU.
==========================================================================*/

#include "../Common.ush"

#ifndef DISPATCH_DESC_MAX_SIZE_DWORDS
#define DISPATCH_DESC_MAX_SIZE_DWORDS 32
#endif // DISPATCH_DESC_MAX_SIZE_DWORDS

#define DISPATCH_DESC_MAX_SIZE_UINT4S (DISPATCH_DESC_MAX_SIZE_DWORDS/4)

uint4 DispatchDescInput[DISPATCH_DESC_MAX_SIZE_UINT4S];
uint DispatchDescSizeDwords;
uint DispatchDescDimensionsOffsetDwords;
uint DimensionsBufferOffsetDwords;
Buffer<uint> DispatchDimensions;
RWBuffer<uint> DispatchDescOutput;

[numthreads(DISPATCH_DESC_MAX_SIZE_DWORDS, 1, 1)]
void RayTracingDispatchDescCS(uint i : SV_DispatchThreadID)
{
	if (i < DispatchDescSizeDwords)
	{
		if (i >= DispatchDescDimensionsOffsetDwords && i < DispatchDescDimensionsOffsetDwords + 3)
		{
			uint DimensionIndex = i - DispatchDescDimensionsOffsetDwords;
			DispatchDescOutput[i] = DispatchDimensions[DimensionsBufferOffsetDwords + DimensionIndex];
		}
		else
		{
			DispatchDescOutput[i] = DispatchDescInput[i/4][i%4];
		}
	}
}
