// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RayTracingDynamicMesh.usf
=============================================================================*/

// Change to force shader compilation of this shader
#pragma message("UESHADERMETADATA_VERSION 74B3439C-C6C2-47F8-B07C-05009AB185D1")

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ComputeShaderUtils.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#define PER_VERTEX_MASK_ENABLED (USE_VERTEX_MASK && VF_SUPPORTS_RAYTRACING_VERTEX_MASK && MATERIALBLENDING_MASKED && RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE)

#if COMPUTESHADER
uint UsingIndirectDraw;
uint MaxNumThreads;
uint NumVertices;
uint MinVertexIndex;
uint PrimitiveId;
uint OutputVertexBaseIndex;
int InstanceId;
float4x4 WorldToInstance;

RWBuffer<float> RWVertexPositions;
int bApplyWorldPositionOffset;

Buffer<uint> IndexBuffer;
uint IndexBufferOffset;

void HideTriangle(uint TriangleIndex, float3 ReferenceLocalPositon)
{
	for (int VertexIndex = 0; VertexIndex < 3; VertexIndex++)
	{		
		const uint OutputVertexIndex = OutputVertexBaseIndex + (MinVertexIndex + TriangleIndex * 3 + VertexIndex) * 3;

		RWVertexPositions[OutputVertexIndex + 0] = ReferenceLocalPositon.x;
		RWVertexPositions[OutputVertexIndex + 1] = ReferenceLocalPositon.y;
		RWVertexPositions[OutputVertexIndex + 2] = ReferenceLocalPositon.z;
	}
}

[numthreads(64, 1, 1)]
void RayTracingDynamicGeometryConverterCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{ 
	uint LinearThreadId = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, 64);

	if (LinearThreadId >= MaxNumThreads) return;

	ResolvedView = ResolveView();

	bool bUsingIndirectDraw = UsingIndirectDraw != 0;
	uint NumActualVertices = bUsingIndirectDraw ? GetNumRayTracingDynamicMeshVerticesIndirect() : NumVertices;

#if PER_VERTEX_MASK_ENABLED
	uint3 Indices;
	Indices.x = IndexBuffer[IndexBufferOffset + LinearThreadId * 3];
	Indices.y = IndexBuffer[IndexBufferOffset + LinearThreadId * 3 + 1];
	Indices.z = IndexBuffer[IndexBufferOffset + LinearThreadId * 3 + 2];

	bool bHideTriangle = false;
	float3 ReferenceLocalPosition = 0.0f;	

	for(int Index = 0; Index < 3; Index++)
	{
		const uint VertexIndex = Indices[Index];
		const uint OutputVertexIndex = OutputVertexBaseIndex + (MinVertexIndex + LinearThreadId * 3 + Index) * 3;
#else
		const uint VertexIndex = LinearThreadId;
		const uint OutputVertexIndex = OutputVertexBaseIndex + (MinVertexIndex + VertexIndex) * 3;
#endif // PER_VERTEX_MASK_ENABLED

		if (VertexIndex < NumActualVertices)
		{
			const uint ActualVertexIndex = MinVertexIndex + VertexIndex;
			FVertexFactoryInput Input = LoadVertexFactoryInputForDynamicUpdate(ActualVertexIndex / 3, ActualVertexIndex % 3, PrimitiveId, InstanceId);

	#if USE_INSTANCING
			Input.InstanceId = InstanceId;
	#elif USE_INSTANCE_CULLING
			Input.DrawInstanceId = InstanceId;
	#endif
			FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);

			float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
			float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);
			FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, TranslatedWorldPosition.xyz, TangentToLocal);

			if (bApplyWorldPositionOffset)
			{
				// We must guarantee that no NaNs are produced by WPO to avoid a possibility of producing invalid BLAS.
				// DXR specification allows NaN positions to mark "inactive" primitives, but only when X component each vertex is NaN.
				// This is impossible to guarantee here, since WPO is evaluated per vertex and not per triangle.
				TranslatedWorldPosition.xyz += MakeFinite(GetMaterialWorldPositionOffset(VertexParameters));
			}

			float3 LocalPosition;
	#if USE_INSTANCING
			//Reverse transform to neutral space
			float3 WorldPosition = TranslatedWorldPosition.xyz - DFHackToFloat(ResolvedView.PreViewTranslation);
			LocalPosition = mul(float4(WorldPosition, 1), WorldToInstance).xyz;
	#elif RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE
			// Move the point back into local space because the RT instance will be placed there
			// This the recommended default behavior so that the transform applied by the VertexFactory and the RT transform are always the same
			float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(VertexFactoryGetWorldToLocal(VFIntermediates), ResolvedView.PreViewTranslation);
			LocalPosition = mul(TranslatedWorldPosition, TranslatedWorldToLocal).xyz;
	#elif RAY_TRACING_DYNAMIC_MESH_IN_WORLD_SPACE
			// There are a few geometry types which natively generate positions in world space and therefore not need any extra processing here.
			// This behavior must be explicitly opted-in to so as to make the code more self-documenting and avoid un-intended mismatched transforms.
			LocalPosition = TranslatedWorldPosition.xyz - DFHackToFloat(ResolvedView.PreViewTranslation);
	#else
	#error "Are you sure you want mesh vertices to be in world space? Please update the VertexFactory to report its requirement!"
			LocalPosition = TranslatedWorldPosition.xyz - DFHackToFloat(ResolvedView.PreViewTranslation);
	#endif

	#if PER_VERTEX_MASK_ENABLED
			FVertexFactoryInterpolantsVSToPS Interpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

			float4 SvPosition = 0;
			FMaterialPixelParameters MaterialPixelParameters = GetMaterialPixelParameters(Interpolants, SvPosition);

			FPixelMaterialInputs PixelMaterialInputs = (FPixelMaterialInputs)0;
			CalcMaterialParameters(MaterialPixelParameters, PixelMaterialInputs, SvPosition, true);
		
			const float Opacity = GetMaterialMask(PixelMaterialInputs);
			if (Opacity < 0.0f)
			{
				bHideTriangle = true;
				ReferenceLocalPosition = LocalPosition;
			}
	#endif		

			RWVertexPositions[OutputVertexIndex + 0] = LocalPosition.x;
			RWVertexPositions[OutputVertexIndex + 1] = LocalPosition.y;
			RWVertexPositions[OutputVertexIndex + 2] = LocalPosition.z;
		}
		else
		{
			RWVertexPositions[OutputVertexIndex + 0] = asfloat(0xFFFFFFFF);
			RWVertexPositions[OutputVertexIndex + 1] = 0;
			RWVertexPositions[OutputVertexIndex + 2] = 0;
		}

#if PER_VERTEX_MASK_ENABLED
	}

	if (bHideTriangle)
	{
		HideTriangle(LinearThreadId, ReferenceLocalPosition);
	}
#endif
}
#endif
