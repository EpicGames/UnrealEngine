// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RayTracingFeedback.usf: Process ray tracing geometry feedback
===============================================================================================*/

#include "../Common.ush"
#include "../ComputeShaderUtils.ush"
#include "/Engine/Shared/RayTracingDebugTypes.h"

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 1
#endif

uint NumGeometries;
uint NumInstances;

StructuredBuffer<uint> GeometryHitCountBuffer;
StructuredBuffer<uint> GeometryHandleBuffer;
StructuredBuffer<uint> AccelerationStructureIndexBuffer;
RWStructuredBuffer<uint> RWGeometryHandleBuffer;
RWBuffer<uint> RWGeometryHandleAllocator;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void RayTracingProcessFeedbackCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	const uint Index = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUP_SIZE);

	if (Index < NumGeometries)
	{
		const uint HitCount = GeometryHitCountBuffer[Index];

		if (HitCount > 0)
		{
			uint WriteIndex;
			InterlockedAdd(RWGeometryHandleAllocator[0], 1, WriteIndex);
			RWGeometryHandleBuffer[WriteIndex] = RWGeometryHandleBuffer[Index];
		}
	}
}

StructuredBuffer<uint> InstanceHitCountBuffer;
RWStructuredBuffer<uint> RWGeometryHitCountBuffer;
StructuredBuffer<FRayTracingInstanceExtraData> InstanceExtraDataBuffer;

[numthreads(THREADGROUP_SIZE, 1, 1)]
void RayTracingUpdateGeometryHitCountCS(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	const uint Index = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUP_SIZE);

	if (Index < NumInstances)
	{
		const uint SceneInstanceIndex = InstanceExtraDataBuffer[Index].SceneInstanceIndex;
		const uint InstanceHitCount = InstanceHitCountBuffer[Index];

		const uint GeometryIndex = AccelerationStructureIndexBuffer[SceneInstanceIndex];

		InterlockedAdd(RWGeometryHitCountBuffer[GeometryIndex], InstanceHitCount);
		// TODO: look into improving this
		RWGeometryHandleBuffer[GeometryIndex] = GeometryHandleBuffer[SceneInstanceIndex];
	}
}
