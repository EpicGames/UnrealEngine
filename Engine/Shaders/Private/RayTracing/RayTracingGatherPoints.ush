// Copyright Epic Games, Inc. All Rights Reserved.

#define MAXIMUM_GATHER_POINTS_PER_PIXEL 32

struct FGatherPoints
{
	float3 TranslatedCreationPoint;
	float3 TranslatedPosition;
	uint2 Irradiance;
};

struct FGatherSample
{
	float3 TranslatedCreationPoint;
	float3 TranslatedPosition;
	float3 Irradiance;
	float Pdf;
};


struct FRejectionCriteria
{
	float3 TranslatedCreationPoint;
	float DistanceThreshold2;
};

uint2 PackIrradiance(float3 Irradiance, float Pdf)
{
	uint2 Packed = (uint2)0;
	Irradiance = ClampToHalfFloatRange(Irradiance);

	// Note: Consider keeping Lumiance(Y) in full range
	Packed.x = f32tof16(Irradiance.x) | (f32tof16(Irradiance.y) << 16);
	Packed.y = f32tof16(Irradiance.z) | (f32tof16(Pdf) << 16);
	return Packed;
}

float3 UnpackIrradiance(uint2 Irradiance, inout float Pdf)
{
	float3 Unpacked;

	Unpacked.x = f16tof32(Irradiance.x & 0xffff);
	Unpacked.y = f16tof32(Irradiance.x >> 16);
	Unpacked.z = f16tof32(Irradiance.y & 0xffff);
	Pdf = f16tof32(Irradiance.y >> 16);

	return Unpacked;
}

FGatherSample ReadGatherSample(StructuredBuffer<FGatherPoints> GatherBuffer, uint BaseIndex, uint SampleIndex, uint ElementCount)
{
	FGatherSample GatherSample = (FGatherSample) 0.0;

	uint Index = BaseIndex + SampleIndex * ElementCount;
	GatherSample.TranslatedCreationPoint = GatherBuffer[Index].TranslatedCreationPoint;
	GatherSample.TranslatedPosition = GatherBuffer[Index].TranslatedPosition;
	GatherSample.Irradiance = UnpackIrradiance(GatherBuffer[Index].Irradiance, GatherSample.Pdf);

	return GatherSample;
}

void WriteGatherSample(RWStructuredBuffer<FGatherPoints> GatherBuffer, uint BaseIndex, uint SampleIndex, uint ElementCount, FGatherSample GatherSample)
{
	uint Index = BaseIndex + SampleIndex * ElementCount;
	GatherBuffer[Index].TranslatedCreationPoint = GatherSample.TranslatedCreationPoint;
	GatherBuffer[Index].TranslatedPosition = GatherSample.TranslatedPosition;
	GatherBuffer[Index].Irradiance = PackIrradiance(GatherSample.Irradiance, GatherSample.Pdf);
}

FRejectionCriteria CreateRejectionCriteria(float3 TranslatedCreationPoint, float DistanceThreshold)
{
	FRejectionCriteria RejectionCriteria;
	RejectionCriteria.TranslatedCreationPoint = TranslatedCreationPoint;
	RejectionCriteria.DistanceThreshold2 = DistanceThreshold * DistanceThreshold;
	return RejectionCriteria;
}

void CreateRejectionMask(
	FRejectionCriteria RejectionCriteria,
	StructuredBuffer<FGatherPoints> GatherBuffer,
	uint BaseIndex,
	uint SampleIndex,
	uint ElementCount,
	int SampleCount,
	out bool bShouldReject[MAXIMUM_GATHER_POINTS_PER_PIXEL]
)
{
	{
		for (int Index = 0; Index < SampleCount; ++Index)
		{
			FGatherSample GatherSample = ReadGatherSample(GatherBuffer, BaseIndex, SampleIndex + Index, ElementCount);
			float3 GPDistance2 = RejectionCriteria.TranslatedCreationPoint - GatherSample.TranslatedCreationPoint;
			bShouldReject[Index] = dot(GPDistance2, GPDistance2) > RejectionCriteria.DistanceThreshold2;
		}
	}
	{
		for (int Index = SampleCount; Index < MAXIMUM_GATHER_POINTS_PER_PIXEL; ++Index)
		{
			bShouldReject[Index] = true;
		}
	}
}
