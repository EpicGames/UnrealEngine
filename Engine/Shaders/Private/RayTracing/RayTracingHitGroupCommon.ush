// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RayTracingHitGroupCommon.ush: common header used in all hit group shaders
=============================================================================*/

#pragma once

#ifndef RAYTRACINGHITGROUPCOMMON_USH_INCLUDED
#define RAYTRACINGHITGROUPCOMMON_USH_INCLUDED // Workarround for UE-66460

#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Shared/RayTracingBuiltInResources.h"

#if NANITE_RAY_TRACING
	#include "../Nanite/NaniteRayTrace.ush"
#endif

#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
#	include "/Engine/Public/NvidiaExtensionsCommon.ush"
#endif

uint3 LoadIndices16Bit(ByteAddressBuffer SourceBuffer, uint OffsetInBytes)
{
	uint3 Result;

	// ByteAddressBuffer loads must be aligned to DWORD boundary.
	// We can load 2 DWORDs (4 SHORTs) at a time and extract 3 SHORTs (first 3 or second 3).
	uint AlignedOffsetInBytes = OffsetInBytes & (~3);
	const uint2 PackedIndices = SourceBuffer.Load2(AlignedOffsetInBytes);

	if (AlignedOffsetInBytes == OffsetInBytes)
	{
		// Extract first 3 SHORTs from 2 DWORDs
		Result[0] = PackedIndices[0] & 0xffff;
		Result[1] = PackedIndices[0] >> 16;
		Result[2] = PackedIndices[1] & 0xffff;
	}
	else
	{
		// Extract second 3 SHORTs from 2 DWORDs
		Result[0] = PackedIndices[0] >> 16;
		Result[1] = PackedIndices[1] & 0xffff;
		Result[2] = PackedIndices[1] >> 16;
	}

	return Result;
}

uint3 LoadIndices32Bit(ByteAddressBuffer SourceBuffer, uint OffsetInBytes)
{
	return SourceBuffer.Load3(OffsetInBytes);
}

float3 LoadVertexPositionFloat3(ByteAddressBuffer SourceBuffer, uint BaseOffsetInBytes, uint Index, uint StrideInBytes)
{
	uint OffsetInBytes = BaseOffsetInBytes + Index * StrideInBytes;
	return asfloat(SourceBuffer.Load3(OffsetInBytes));
}

// Explicitly declare "system" resources that will be available to all hit group shaders.

FTriangleBaseAttributes LoadTriangleBaseAttributes(
	ByteAddressBuffer IndexBuffer, uint IndexBufferOffsetInBytes, uint IndexBufferStride, 
	ByteAddressBuffer VertexBuffer, uint VertexBufferOffsetInBytes, uint VertexBufferStride, 
	uint PrimitiveId)
{
	FTriangleBaseAttributes Result = (FTriangleBaseAttributes)0;

	// Fetch vertex indices and positions, then compute local space normal and transform it to world space

	const uint BaseIndex = PrimitiveId * 3;

	if (IndexBufferStride == 0)
	{
		// Non-indexed geometry (implicit triangle list indices)
		Result.Indices = uint3(BaseIndex, BaseIndex + 1, BaseIndex + 2);
	}
	else if (IndexBufferStride == 2)
	{
		Result.Indices = LoadIndices16Bit(IndexBuffer, IndexBufferOffsetInBytes + BaseIndex * IndexBufferStride);
	}
	else
	{
		Result.Indices = LoadIndices32Bit(IndexBuffer, IndexBufferOffsetInBytes + BaseIndex * IndexBufferStride);
	}

	// Fetch vertex positions (in local space)
	// #dxr_todo: UE-72160 handle various vertex formats, for now only supporting float3
	Result.LocalPositions[0] = LoadVertexPositionFloat3(VertexBuffer, VertexBufferOffsetInBytes, Result.Indices[0], VertexBufferStride);
	Result.LocalPositions[1] = LoadVertexPositionFloat3(VertexBuffer, VertexBufferOffsetInBytes, Result.Indices[1], VertexBufferStride);
	Result.LocalPositions[2] = LoadVertexPositionFloat3(VertexBuffer, VertexBufferOffsetInBytes, Result.Indices[2], VertexBufferStride);

	return Result;
}

FTriangleBaseAttributes LoadTriangleBaseAttributes(uint PrimitiveId)
{
	uint IndexBufferOffsetInBytes = HitGroupSystemRootConstants.IndexBufferOffsetInBytes;
	uint IndexBufferStride = HitGroupSystemRootConstants.GetIndexStride();
	uint VertexStride = HitGroupSystemRootConstants.GetVertexStride();
	uint VertexBufferOffsetInBytes = 0; // Base offset is already applied when the buffer is bound to the hit group
	return LoadTriangleBaseAttributes(HitGroupSystemIndexBuffer, IndexBufferOffsetInBytes, IndexBufferStride, HitGroupSystemVertexBuffer, VertexBufferOffsetInBytes, VertexStride, PrimitiveId);
}

uint GetInstanceUserData()
{
	return InstanceID();
}

uint GetHitGroupUserData()
{
	return HitGroupSystemRootConstants.UserData;
}

float3 GetGeometryNormalFromTriangleBaseAttributes(uint PrimitiveIndex, uint ClusterID)
{
	float3 LocalEdges[2];
	
#if NANITE_RAY_TRACING
	const uint GPUSceneInstanceIndex = GetInstanceUserData();

	FInstanceSceneData InstanceData = GetInstanceSceneData(GPUSceneInstanceIndex);
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	if (UsingCLAS && PrimitiveData.NaniteResourceID != uint(-1))
	{
		const uint AssemblyTransformIndex = NANITE_MAX_ASSEMBLY_TRANSFORMS;

		const uint PageIndex = ClusterID & NANITE_MAX_GPU_PAGES_MASK;
		const uint ClusterIndex = (ClusterID >> NANITE_MAX_GPU_PAGES_BITS) & NANITE_MAX_CLUSTERS_PER_PAGE_MASK;

		GetNaniteTriangleEdges(
			PageIndex,
			ClusterIndex,
			PrimitiveIndex,
			AssemblyTransformIndex,
			PrimitiveData.NaniteAssemblyTransformOffset,
			LocalEdges[0],
			LocalEdges[1]);
	}
	else
#endif // PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	if (PrimitiveData.NaniteRayTracingDataOffset != uint(-1))
	{
		GetNaniteTriangleEdges(PrimitiveData.NaniteRayTracingDataOffset + HitGroupSystemRootConstants.FirstPrimitive + PrimitiveIndex, PrimitiveData.NaniteAssemblyTransformOffset, LocalEdges[0], LocalEdges[1]);
	}
	else
#endif // NANITE_RAY_TRACING
	{
		FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(PrimitiveIndex);
		
		LocalEdges[0] = Tri.LocalPositions[1] - Tri.LocalPositions[0];
		LocalEdges[1] = Tri.LocalPositions[2] - Tri.LocalPositions[0];
	}
	
	float3x3 InverseTranspose3x3 = (float3x3)WorldToObject4x3();
	float3 LocalNormal = cross(LocalEdges[1], LocalEdges[0]);
	float3 WorldNormal = normalize(mul(InverseTranspose3x3, LocalNormal));

	return WorldNormal;
}

uint GetClusterID()
{
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS && NANITE_RAY_TRACING
	return UsingCLAS ? NvRtGetClusterID() : 0;
#else
	return 0;
#endif
}

#endif // RAYTRACINGHITGROUPCOMMON_USH_INCLUDED // Workarround for UE-66460
