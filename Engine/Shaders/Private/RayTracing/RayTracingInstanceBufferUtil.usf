// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RayTracingInstanceCopy.usf: Build ray tracing instances in the GPU 
===============================================================================================*/

#include "RayTracingCommon.ush"
#include "../SceneData.ush"
#include "../ComputeShaderUtils.ush"
#include "../Common.ush"
#include "/Engine/Shared/RayTracingDebugTypes.h"

#if NANITE_RAYTRACING
#	include "/Engine/Shared/NaniteRayTracingDefinitions.h"
#endif // NANITE_RAYTRACING

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 1
#endif

struct FRayTracingInstanceDescriptor
{
	uint GPUSceneInstanceOrTransformIndex;
	uint OutputDescriptorIndex;
	uint AccelerationStructureIndex;
	uint InstanceId;
	uint InstanceMaskAndFlags;
	uint InstanceContributionToHitGroupIndex;
	uint SceneInstanceIndexAndApplyLocalBoundsTransform;
};

uint MaxNumInstances;
uint NumGroups;
uint NumInstanceDescriptors;
uint BaseGroupDescriptorIndex;
uint BaseInstanceDescriptorIndex;

float3 PreViewTranslationHigh;
float3 PreViewTranslationLow;

RWStructuredBuffer<FPlatformRayTracingInstanceDescriptor> OutPlatformInstanceDescriptors;

#if USE_SEPARATE_HIT_CONTRIBUTION_BUFFER
RWStructuredBuffer<uint> OutHitGroupContributions;
#endif

StructuredBuffer<uint> InstanceGroupDescriptors;
StructuredBuffer<FRayTracingInstanceDescriptor> InstanceDescriptors;
ByteAddressBuffer AccelerationStructureAddresses;
// Transforms are float3x4 row-major matrices stored as 3 float4
// because of FXC matrix packing issues when using StructuredBuffer<float3x4>
StructuredBuffer<float4> InstanceTransforms;

// Extra per instance data
RWStructuredBuffer<FRayTracingInstanceExtraData> RWInstanceExtraData;

#if OUTPUT_STATS
RWStructuredBuffer<uint> RWOutputStats;
uint OutputStatsOffset;
#endif

ByteAddressBuffer NaniteBLASData;

#if GPU_CULLING

float CullingRadius;
float FarFieldCullingRadius;
float AngleThresholdRatioSq;
float3 ViewOrigin;
uint CullingMode;
uint CullUsingGroups;
int bUseDistanceToBox;

bool CullInstance(FInstanceSceneData InstanceSceneData, FPrimitiveSceneData PrimitiveData, float4x4 LocalToTranslatedWorld4x4, bool bFarFieldInstance)
{
	if ((InstanceSceneData.Flags & INSTANCE_SCENE_DATA_FLAG_HIDDEN) != 0u)
	{
		return true;
	}

	bool bIsVisible = true;

	const float3 LocalBoxCenter = InstanceSceneData.LocalBoundsCenter.xyz;
	const float3 LocalBoxExtent = InstanceSceneData.LocalBoundsExtent.xyz;
	const float4x4 LocalToTranslatedWorld = LocalToTranslatedWorld4x4;
	const float3 NonUniformScale = InstanceSceneData.NonUniformScale.xyz;
	
	const float3 CenterTranslatedWorld = mul(float4(LocalBoxCenter, 1.0f), LocalToTranslatedWorld).xyz;
	const float Radius = length(LocalBoxExtent * NonUniformScale);

	const float InstanceDrawDistSq = length2(CenterTranslatedWorld - ViewOrigin);

	float DistanceToInstanceSq;
	if (bUseDistanceToBox != 0)
	{
		float4 Temp = 0.0f;
		Temp += abs(LocalBoxExtent.xxxx * LocalToTranslatedWorld[0]);
		Temp += abs(LocalBoxExtent.yyyy * LocalToTranslatedWorld[1]);
		Temp += abs(LocalBoxExtent.zzzz * LocalToTranslatedWorld[2]);

		float3 WorldAABBExtent = Temp.xyz;
		DistanceToInstanceSq = ComputeSquaredDistanceFromBoxToPoint(CenterTranslatedWorld, WorldAABBExtent, ViewOrigin);
	}
	else
	{
		DistanceToInstanceSq = max(sqrt(InstanceDrawDistSq) - Radius, 0.0f);
		DistanceToInstanceSq *= DistanceToInstanceSq;
	}
	
	if (bFarFieldInstance)
	{
		bIsVisible = DistanceToInstanceSq <= Square(FarFieldCullingRadius);
	}
	else
	{
		const bool bIsVisibleDistance = DistanceToInstanceSq <= Square(CullingRadius);
		const bool bIsVisibleAngle = (Square(Radius) / InstanceDrawDistSq) >= AngleThresholdRatioSq;
		
		// Only culling modes 2 & 3 are supported for now
		if (CullingMode == 3)
		{
			bIsVisible = bIsVisibleDistance && bIsVisibleAngle;
		}
		else if (CullingMode == 2)
		{
			bIsVisible = bIsVisibleDistance || bIsVisibleAngle;	
		}		
		
		const bool bDrawDistance = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_DRAW_DISTANCE_CULL) != 0u;
		if (bIsVisible && bDrawDistance)
		{
			const float MinDistanceSq = PrimitiveData.InstanceDrawDistanceMinMaxSquared.x;
			const float MaxDistanceSq = PrimitiveData.InstanceDrawDistanceMinMaxSquared.y;

			bIsVisible = InstanceDrawDistSq >= MinDistanceSq && InstanceDrawDistSq <= MaxDistanceSq;
		}
	}	
	return !bIsVisible;
}

#endif // GPU_CULLING

void ProcessInstance(FRayTracingInstanceDescriptor InputDesc)
{
	const FDFVector3 PreViewTranslation = MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	bool bCulled = false;

#if USE_GPUSCENE
	FInstanceSceneData InstanceSceneData = GetInstanceSceneData(InputDesc.GPUSceneInstanceOrTransformIndex);
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceSceneData.PrimitiveId);

	float4x4 LocalToTranslatedWorld4x4 = DFFastToTranslatedWorld(InstanceSceneData.LocalToWorld, PreViewTranslation);

	const bool bFarFieldInstance = PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_FAR_FIELD;

#if GPU_CULLING
	const bool bHasGroupId = PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_HAS_GROUPID;
	if (!bHasGroupId || !CullUsingGroups)
	{
		bCulled = CullInstance(InstanceSceneData, PrimitiveData, LocalToTranslatedWorld4x4, bFarFieldInstance);
	}
#endif // GPU_CULLING

	if ((InputDesc.SceneInstanceIndexAndApplyLocalBoundsTransform & 0x80000000) != 0)
	{
		float4x4 ProcT;
		ProcT[0] = float4(InstanceSceneData.LocalBoundsExtent.x * 2, 0, 0, 0);
		ProcT[1] = float4(0, InstanceSceneData.LocalBoundsExtent.y * 2, 0, 0);
		ProcT[2] = float4(0, 0, InstanceSceneData.LocalBoundsExtent.z * 2, 0);
		ProcT[3] = float4(InstanceSceneData.LocalBoundsCenter, 1);

		LocalToTranslatedWorld4x4 = mul(ProcT, LocalToTranslatedWorld4x4);
	}

	LocalToTranslatedWorld4x4 = transpose(LocalToTranslatedWorld4x4);

	float3x4 LocalToTranslatedWorld;
	LocalToTranslatedWorld[0] = LocalToTranslatedWorld4x4[0];
	LocalToTranslatedWorld[1] = LocalToTranslatedWorld4x4[1];
	LocalToTranslatedWorld[2] = LocalToTranslatedWorld4x4[2];
#else // !USE_GPUSCENE
	float3x4 LocalToTranslatedWorld;
	LocalToTranslatedWorld[0] = InstanceTransforms[InputDesc.GPUSceneInstanceOrTransformIndex * 3 + 0];
	LocalToTranslatedWorld[1] = InstanceTransforms[InputDesc.GPUSceneInstanceOrTransformIndex * 3 + 1];
	LocalToTranslatedWorld[2] = InstanceTransforms[InputDesc.GPUSceneInstanceOrTransformIndex * 3 + 2];
#endif // !USE_GPUSCENE

	if (bCulled)
	{
#	if COMPACT_OUTPUT
		// when using compaction don't need to write inactive instances
		return;
#	endif
	}
	else
	{
#	if OUTPUT_STATS
		uint CompactedIndex;
#		if USE_WAVE_OPS
			WaveInterlockedAddScalar_(RWOutputStats[OutputStatsOffset + 0], 1, CompactedIndex);
#		else
			InterlockedAdd(RWOutputStats[OutputStatsOffset + 0], 1, CompactedIndex);
#		endif
#		if COMPACT_OUTPUT
			InputDesc.OutputDescriptorIndex = CompactedIndex;
#		endif // COMPACT_OUTPUT
#	endif // OUTPUT_STATS
	}

	if (InputDesc.OutputDescriptorIndex >= MaxNumInstances)
	{
		return;
	}

#if OUTPUT_INSTANCE_EXTRA_DATA
	if (!bCulled)
	{
		FRayTracingInstanceExtraData ExtraData;
		ExtraData.SceneInstanceIndex = InputDesc.SceneInstanceIndexAndApplyLocalBoundsTransform & 0x7fffffff;
#if USE_GPUSCENE
		ExtraData.GPUSceneInstanceId = InputDesc.GPUSceneInstanceOrTransformIndex;
#else
		ExtraData.GPUSceneInstanceId = -1;
#endif
		RWInstanceExtraData[InputDesc.OutputDescriptorIndex] = ExtraData;
	}
#endif // OUTPUT_INSTANCE_EXTRA_DATA

	uint2 BlasAddress = 0;

	if (!bCulled)
	{
		BlasAddress = AccelerationStructureAddresses.Load2(InputDesc.AccelerationStructureIndex * 8);
		
#if NANITE_RAYTRACING
		if (PrimitiveData.NaniteResourceID != uint(-1))
		{
			FNaniteRayTracingASData BLASData = NaniteBLASData.Load<FNaniteRayTracingASData>(PrimitiveData.NaniteResourceID * uint(sizeof(FNaniteRayTracingASData)));
			BlasAddress = UnpackUlongType(BLASData.Address);
		}
#endif // NANITE_RAYTRACING
	}

	OutPlatformInstanceDescriptors[InputDesc.OutputDescriptorIndex] = BuildPlatformRayTracingInstanceDesc(
		InputDesc.InstanceMaskAndFlags & 0xFF,
		InputDesc.InstanceId,
		TranslateRayTracingInstanceFlags((InputDesc.InstanceMaskAndFlags >> 8) & 0xFF),
		InputDesc.InstanceContributionToHitGroupIndex,
		LocalToTranslatedWorld,
		BlasAddress);
		
#if USE_SEPARATE_HIT_CONTRIBUTION_BUFFER
	OutHitGroupContributions[InputDesc.OutputDescriptorIndex] = InputDesc.InstanceContributionToHitGroupIndex;
#endif
}

[numthreads(THREADGROUP_SIZE, 1, 1)]
void RayTracingBuildInstanceBufferCS(uint GroupThreadIndex : SV_GroupIndex, uint3 GroupId : SV_GroupID)
{
	const uint GroupIndex = GetUnWrappedDispatchGroupId(GroupId);
	const uint Index = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, THREADGROUP_SIZE);

#if SUPPORT_INSTANCE_GROUPS
	if (GroupIndex >= NumGroups)
	{
		return;
	}

	const uint PackedInstanceGroupDesc = InstanceGroupDescriptors[BaseGroupDescriptorIndex + GroupIndex];

	const bool bIsRLEPacked = (PackedInstanceGroupDesc & 0x80000000) != 0;
	const bool bIncrementUserDataPerInstance = (PackedInstanceGroupDesc & 0x40000000) != 0;
	const uint GroupBaseInstanceDescIndex = PackedInstanceGroupDesc & 0x3FFFFFFF;

	uint InstanceDescIndex = GroupBaseInstanceDescIndex;

	if (!bIsRLEPacked)
	{
		InstanceDescIndex += GroupThreadIndex;
	}

	if (InstanceDescIndex >= NumInstanceDescriptors)
	{
		return;
	}

	FRayTracingInstanceDescriptor InputDesc = InstanceDescriptors[BaseInstanceDescriptorIndex + InstanceDescIndex];

	if (bIsRLEPacked)
	{
		InputDesc.GPUSceneInstanceOrTransformIndex += GroupThreadIndex;
		InputDesc.OutputDescriptorIndex += GroupThreadIndex;

		if (bIncrementUserDataPerInstance)
		{
			InputDesc.InstanceId += GroupThreadIndex;
		}
	}

	ProcessInstance(InputDesc);
#else
	if (Index >= NumInstanceDescriptors)
	{
		return;
	}

	FRayTracingInstanceDescriptor InputDesc = InstanceDescriptors[BaseInstanceDescriptorIndex + Index];
	ProcessInstance(InputDesc);
#endif
}
