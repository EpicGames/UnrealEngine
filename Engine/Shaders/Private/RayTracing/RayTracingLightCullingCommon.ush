// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RayTracingLightCullingCommon.ush: Common functions for culling lights.
===============================================================================================*/

#pragma once

#include "../Visualization.ush"

// *************** LIGHT CULLING *********************************
// Use a 2D grid built along the longest axes of the bounding boxes of all finite light sources.

struct FCulledLightList
{
	uint NumLights;
	uint LightGridOffset;

	static FCulledLightList Create(float3 TranslatedWorldPos)
	{
		FCulledLightList Result = (FCulledLightList) 0;
		Result.NumLights = RaytracingLightGridData.SceneInfiniteLightCount;
		Result.LightGridOffset = ~0u;
		
		const float3 SceneMin = RaytracingLightGridData.SceneLightsTranslatedBoundMin;
		const float3 SceneMax = RaytracingLightGridData.SceneLightsTranslatedBoundMax;
		if (all(SceneMin <= TranslatedWorldPos) &&
			all(TranslatedWorldPos <= SceneMax))
		{
			float3 P = TranslatedWorldPos - SceneMin;
			float3 D = SceneMax - SceneMin;
			int2 UVx = int2(floor(RaytracingLightGridData.LightGridResolution * P.yz / D.yz));
			int2 UVy = int2(floor(RaytracingLightGridData.LightGridResolution * P.xz / D.xz));
			int2 UVz = int2(floor(RaytracingLightGridData.LightGridResolution * P.xy / D.xy));

			uint LightGridNx = RaytracingLightGridData.LightGrid.Load(int4(UVx, 0, 0));
			uint LightGridNy = RaytracingLightGridData.LightGrid.Load(int4(UVy, 1, 0));
			uint LightGridNz = RaytracingLightGridData.LightGrid.Load(int4(UVz, 2, 0));
			uint LightGridN = LightGridNx;
			uint LightGridOffset = RaytracingLightGridData.LightGridMaxCount * (0 + 3 * (UVx.x + UVx.y * RaytracingLightGridData.LightGridResolution));
			if ((RaytracingLightGridData.LightGridAxis >= 3 && LightGridNy < LightGridN) || RaytracingLightGridData.LightGridAxis == 1)
			{
				LightGridN = LightGridNy;
				LightGridOffset = RaytracingLightGridData.LightGridMaxCount * (1 + 3 * (UVy.x + UVy.y * RaytracingLightGridData.LightGridResolution));
			}
			if ((RaytracingLightGridData.LightGridAxis >= 3 && LightGridNz < LightGridN) || RaytracingLightGridData.LightGridAxis == 2)
			{
				LightGridN = LightGridNz;
				LightGridOffset = RaytracingLightGridData.LightGridMaxCount * (2 + 3 * (UVz.x + UVz.y * RaytracingLightGridData.LightGridResolution));
			}
			Result.LightGridOffset = LightGridOffset;
			Result.NumLights += LightGridN;
		}
		return Result;
	}

	uint GetNumLights()
	{
		return NumLights;
	}

	uint GetLightIndex(int LightNum, out uint Valid)
	{
		if (LightNum >= NumLights)
		{
			Valid = 0;
			return 0;
		}
		Valid = 1;
		if (LightNum >= RaytracingLightGridData.SceneInfiniteLightCount)
		{
			return RaytracingLightGridData.LightGridData[LightGridOffset + LightNum - RaytracingLightGridData.SceneInfiniteLightCount];
		}
		return LightNum;
	}

	float3 Visualize(int VisualizeMode)
	{
		const float3 OutsideColor = 0.18;
		const float3 EmptyColor = 0.36;

		if (LightGridOffset != ~0u)
		{
			switch (VisualizeMode)
			{
				case 2:
				{
					// color by unique light list
					uint H = 0;
					for (int Index = 0; Index < NumLights; Index++)
					{
						uint Valid = 0;
						uint LightId = GetLightIndex(Index, Valid);
						if (Valid)
						{
							H += LightId;
							H ^= H >> 16;
							H *= 0xa812d533;
							H ^= H >> 15;
							H *= 0xb278e4ad;
							H ^= H >> 17;
						}
					}
					return HUEtoRGB((H & 0xFFFFFF) * 5.96046447754e-08);
				}
				default: 
				{
					// default mode - color by light count
					uint N = NumLights - RaytracingLightGridData.SceneInfiniteLightCount;
					if (N < 1)
					{
						return EmptyColor;
					}
					float Max = RaytracingLightGridData.LightGridMaxCount;
					float t = saturate(float(N) / Max);
					return BlueGreenRedColorMap(t);
				}
			}
		}
		// outside light grid bounds
		return OutsideColor;
	}
};

