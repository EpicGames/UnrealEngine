// Copyright Epic Games, Inc. All Rights Reserved.


// base light function material shader, so it can be folded where apprpriate
// see LightFunctionCommon and LightFunctionPixelShader for additional information
float3 GetRayTracingLightFunction(float3 TranslatedWorldPosition)
{
	float3 LightVector;

	{
		// Not really translated, just dealing with the fact that common code uses this convention
		float4 Hom = mul(float4(TranslatedWorldPosition, 1), LightFunctionTranslatedWorldToLight);
		LightVector = Hom.xyz / Hom.w;
	}

	// Calculate radial view distance for stable fading
	float ViewDistance = GetDistanceToCameraFromViewVector(View.TranslatedWorldCameraOrigin - TranslatedWorldPosition);

	float3 Color = GetLightFunctionColor(LightVector, TranslatedWorldPosition);

	float DistanceFadeAlpha = saturate((LightFunctionParameters2.x - ViewDistance) / (LightFunctionParameters2.x * .2f));
	// Fade to disabled based on LightFunctionFadeDistance
	Color = lerp(LightFunctionParameters2.y, Color, DistanceFadeAlpha);

	// Fade to disabled based on ShadowFadeFraction
	Color = lerp(LightFunctionParameters2.y, Color, LightFunctionParameters.y);

	return Color;
}


