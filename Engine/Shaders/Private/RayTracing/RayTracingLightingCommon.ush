// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RayTracingCommon.ush"

#include "../DeferredLightingCommon.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush" 
#include "../LightShaderParameters.ush"
#include "RayTracingDirectionalLight.ush"
#include "RayTracingRectLight.ush"
#include "RayTracingSphereLight.ush"
#include "RayTracingCapsuleLight.ush"
#include "RayTracingSkyLightEvaluation.ush"
#include "RayTracingLightCullingCommon.ush"
#include "/Engine/Shared/LightDefinitions.h"
#include "/Engine/Shared/RayTracingTypes.h"

#ifndef ENABLE_FAR_FIELD_TRACING
#define ENABLE_FAR_FIELD_TRACING 0
#endif // ENABLE_FAR_FIELD_TRACING

FDeferredLightData GetRayTracingDeferredLightData(
	int LightIndex,
	bool bIndirectLighting,
	inout uint OutLightType,
	inout uint OutLightMissShaderIndex)
{
	FDeferredLightData LightData = (FDeferredLightData)0;
	FRTLightingData RayTracingLightData = RaytracingLightGridData.SceneLights[LightIndex];

	const uint LightType = RayTracingLightData.Type;

	LightData.TranslatedWorldPosition = RayTracingLightData.TranslatedLightPosition;
	LightData.InvRadius = RayTracingLightData.InvRadius;
	LightData.Color = RayTracingLightData.LightColor * (bIndirectLighting ? RayTracingLightData.IndirectLightScale : 1.0f);
	LightData.FalloffExponent = RayTracingLightData.FalloffExponent;
	LightData.Direction = RayTracingLightData.Direction;
	LightData.Tangent = RayTracingLightData.Tangent;
	LightData.SpotAngles = RayTracingLightData.SpotAngles;
	LightData.SourceRadius = RayTracingLightData.SourceRadius;
	LightData.SourceLength = RayTracingLightData.SourceLength;
	LightData.SoftSourceRadius = RayTracingLightData.SoftSourceRadius;
	LightData.DiffuseScale = UnpackFloat2FromUInt(RayTracingLightData.DiffuseSpecularScale).x;
	LightData.SpecularScale = UnpackFloat2FromUInt(RayTracingLightData.DiffuseSpecularScale).y;
	LightData.RectLightData.BarnCosAngle = RayTracingLightData.RectLightBarnCosAngle;
	LightData.RectLightData.BarnLength = RayTracingLightData.RectLightBarnLength;
	LightData.DistanceFadeMAD = UnpackFloat2FromUInt(RayTracingLightData.DistanceFadeMAD);

	LightData.ShadowMapChannelMask = float4(0, 0, 0, 0);
	LightData.ShadowedBits = 0; // Not lit dynamic shadows 
	LightData.ContactShadowLength = 0.0;
	LightData.ContactShadowLengthInWS = false;
	LightData.ContactShadowCastingIntensity = 1.0f;
	LightData.ContactShadowNonCastingIntensity = 0.0f;

	LightData.bRadialLight = (LightType != LIGHT_TYPE_DIRECTIONAL);
	LightData.bSpotLight = (LightType == LIGHT_TYPE_SPOT);
	LightData.bRectLight = (LightType == LIGHT_TYPE_RECT);

	if (LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		LightData.bInverseSquared = false;
	}
	else
	{
		LightData.bInverseSquared = LightData.FalloffExponent == 0;
	}

	LightData.RectLightData.AtlasData.AtlasUVOffset = RayTracingLightData.RectLightAtlasUVOffset;
	LightData.RectLightData.AtlasData.AtlasUVScale  = RayTracingLightData.RectLightAtlasUVScale;
	LightData.RectLightData.AtlasData.AtlasMaxLevel = RayTracingLightData.RectLightAtlasMaxLevel;

	LightData.IESAtlasIndex = RayTracingLightData.IESAtlasIndex;

	LightData.HairTransmittance = InitHairTransmittanceData();

	OutLightType = LightType;
	OutLightMissShaderIndex = RayTracingLightData.LightMissShaderIndex;

	return LightData;
}

FDeferredLightData GetRayTracingDeferredLightData(
	int LightIndex,
	bool bIndirectLighting,
	inout uint OutLightType)
{
	uint LightMissShaderIndex = 0;
	return GetRayTracingDeferredLightData(LightIndex, bIndirectLighting, OutLightType, LightMissShaderIndex);
}

float3 GenerateReflectedRayDirection(
	float3 IncidentDirection,
	float3 WorldNormal,
	float Roughness,
	float2 RandSample
)
{
	float3 RayDirection;
	if (Roughness < 0.001) //ReflectionSmoothClamp)
	{
		RayDirection = reflect(IncidentDirection, WorldNormal);
	}
	else
	{
		float3 N = WorldNormal;
		float3 V = -IncidentDirection;
		float2 E = RandSample;

		float3x3 TangentBasis = GetTangentBasis(N);
		float3 TangentV = mul(TangentBasis, V);

		float NoV = saturate(dot(V, WorldNormal));

		float4 Sample = ImportanceSampleVisibleGGX(E, Pow2(Roughness), TangentV);

		float3 H = mul(Sample.xyz, TangentBasis);
		float3 L = 2 * dot(V, H) * H - V;

		RayDirection = L;
	}

	return RayDirection;
}

void TraceShadowRayMissShaderLighting(
	in FRayDesc Ray,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	in RaytracingAccelerationStructure TLAS,
	in uint MissShaderIndex,
	inout FPackedMaterialClosestHitPayload PackedPayload)
{
	TraceRay
	(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		RAY_TRACING_SHADER_SLOT_SHADOW,
		RAY_TRACING_NUM_SHADER_SLOTS,
		MissShaderIndex,
		Ray.GetNativeDesc(),
		PackedPayload
	);
}

// Returns xyz: Direction, w: RayLength
float4 SampleAreaLightDirection(
	in FDeferredLightData LightData, 
	in float3 TranslatedWorldPosition,
	in float3 WorldNormal,
	in uint LightType, 
	inout FRandomSequence RandSequence )
{
	float3 ShadowRayDirection = 0.0;

	float3 RayOrigin = float3(0,0,0);
	float RayTMin = 0;
	float RayTMax = 0;
	float RayPdf = 0;

	float2 RandSample = RandSequence.Get2D();

	FLightShaderParameters LightParameters;
	LightParameters.TranslatedWorldPosition = LightData.TranslatedWorldPosition;
	LightParameters.SpotAngles = LightData.SpotAngles;
	LightParameters.SourceRadius = LightData.SourceRadius;
	LightParameters.SourceLength = LightData.SourceLength;
	LightParameters.Tangent = LightData.Tangent;
	LightParameters.Direction = LightData.Direction;

	if (LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		float ShadowSourceAngleFactor = LightData.RectLightData.BarnCosAngle;
		LightParameters.SourceRadius *= ShadowSourceAngleFactor;
		GenerateDirectionalLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ ShadowRayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax);
	}
	else if (LightType == LIGHT_TYPE_POINT || LightType == LIGHT_TYPE_SPOT)
	{
		// NOTE: Spot cone is being checked before calling this function, so we know we are inside the cone
		if (LightParameters.SourceLength > 0.0)
		{
			GenerateCapsuleLightOcclusionRayWithSolidAngleSampling(
				LightParameters,
				TranslatedWorldPosition, WorldNormal,
				RandSample,
				/* out */ RayOrigin,
				/* out */ ShadowRayDirection,
				/* out */ RayTMin,
				/* out */ RayTMax,
				/* out */ RayPdf);
		}
		else
		{
			GenerateSphereLightOcclusionRayWithSolidAngleSampling(
				LightParameters,
				TranslatedWorldPosition, WorldNormal,
				RandSample,
				/* out */ RayOrigin,
				/* out */ ShadowRayDirection,
				/* out */ RayTMin,
				/* out */ RayTMax,
				/* out */ RayPdf);
		}
	}
	else if (LightType == LIGHT_TYPE_RECT)
	{
		// fill in extra parameters used only for rect lights
		LightParameters.RectLightBarnLength = LightData.RectLightData.BarnLength;
		LightParameters.RectLightBarnCosAngle = LightData.RectLightData.BarnCosAngle;
		GenerateRectLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ ShadowRayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax,
			/* out */ RayPdf);
	}

	return float4(ShadowRayDirection, RayTMax);
}

float3 ComputeIndirectLighting(
	in float3 TranslatedWorldPosition,
	in float3 ViewDirection,
	in RaytracingAccelerationStructure TLAS,
	in uint2 PixelCoord,
	in FPackedMaterialClosestHitPayload Payload,
	in bool bRayTraceSkyLightContribution,
	in bool bDecoupleSampleGeneration)
#if SUBSTRATE_GBUFFER_FORMAT==1
{
	// SUBSTRATE_TODO
	return 0;
}
#else
{
	float3 IndirectLighting = float3(0.0f, 0.0f, 0.0f);

	// Payload indirect irradiance contribution
	float3 DiffuseColor = Payload.GetDiffuseColor();
	if (Payload.GetShadingModelID() ==  SHADINGMODELID_CLOTH)
	{
		float4 CustomData = Payload.GetCustomData();
		DiffuseColor += CustomData.rgb * CustomData.a;
	}
	IndirectLighting += DiffuseColor * Payload.GetIndirectIrradiance();

	// Ray traced sky light contribution
	if (bRayTraceSkyLightContribution)
	{
		FGBufferData GBufferData = GetGBufferDataFromPayload(Payload);

		// Evaluate the Sky Light at the surface point
		const bool bGBufferSampleOrigin = false;
		const float DeviceZ = 0.0f; // No camera related depth needed since sample is not from g-buffer
		float3 ExitantRadiance;
		float3 DiffuseExitantRadiance;
		float AmbientOcclusion;
		float HitDistance;

		SkyLightEvaluate(
			PixelCoord,
			SkyLight.SamplesPerPixel,
			TranslatedWorldPosition,
			GBufferData.WorldNormal,
			ViewDirection,
			GBufferData,
			TLAS,
			bGBufferSampleOrigin,
			DeviceZ,
			bDecoupleSampleGeneration,
			ExitantRadiance,
			DiffuseExitantRadiance,
			AmbientOcclusion,
			HitDistance);

		// Add the diffuse exitant radiance to the contribution
		IndirectLighting += DiffuseExitantRadiance;
	}

	return IndirectLighting;
}
#endif

bool HasBackfaceDiffuse(FPackedMaterialClosestHitPayload Payload)
{
	#if SUBSTRATE_GBUFFER_FORMAT==1
	{
		#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE == 0
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(0,0,0);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Payload.SubstrateData, SubstrateAddressing, Payload.SubstrateData);
		const uint BSDFType = SubstratePixelHeader.SubstrateGetBSDFType();
		return BSDFType == SUBSTRATE_BSDF_TYPE_SLAB && SubstratePixelHeader.HasSubsurface();
		#else
		return false;
		#endif
	}
	#else
	{
		const uint ShadingModelID = Payload.GetShadingModelID();
		return ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE 
			|| ShadingModelID == SHADINGMODELID_SUBSURFACE;
	}
	#endif
}

float3 ComputeDirectLighting(
	in float3 TranslatedWorldPosition,
	in float3 ViewDirection,
	in FRayCone RayCone,
	in RaytracingAccelerationStructure TLAS,
	in RaytracingAccelerationStructure FarFieldTLAS,
	inout FPackedMaterialClosestHitPayload Payload,
	inout FRandomSequence RandSequence,
	in uint ShadowsType,
	in uint ShadowsTranslucencyType,
	float ShadowMaxNormalBias,
	float MaxTraceDistance)
{
	float3 DirectLighting = (float3)0;

	// Repurpose some fields in the material payload to pass parameters into lighting miss shader.
	// Backup the values that will be aliased with other entries
	float3 OldRadiance = Payload.GetRadiance();
#if SUBSTRATE_GBUFFER_FORMAT==1
	uint OldTransmittancePreCoverage = Payload.TransmittancePreCoverage;
	uint OldSceneInstanceIndex = Payload.SceneInstanceIndex;
	uint OldFlags = Payload.Flags;
#else
	uint2 OldIndirectIrradiance = uint2(Payload.PackedIndirectIrradiance[0], Payload.PackedIndirectIrradiance[1]);
	uint OldFlags = Payload.FlagsAndMipBias;
#endif
	uint OldRayCone = Payload.PackedRayCone;
	Payload.SetRadiance(float3(0, 0, 0));
	Payload.SetRayDirection(ViewDirection);

	FCulledLightList CullData = FCulledLightList::Create(TranslatedWorldPosition);
	const uint LightCount = CullData.GetNumLights();

	for (uint Index = 0; WaveActiveAnyTrue(Index < LightCount); Index++)
	{
		uint Lit = 0;
		uint LightIndex = 0;

		LightIndex = CullData.GetLightIndex(Index, Lit);

		uint LightType = 0;
		uint LightMissShaderIndex = 0;
		FDeferredLightData LightData = GetRayTracingDeferredLightData(LightIndex, Payload.IsIndirectLightingRay(), LightType, LightMissShaderIndex);

		float3 ShadowRayDirection;
		// ToLight should not be normalized because its length is used to compute the shadow ray TMax 
		float3 ToLight = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
		float LightMask = 1.0;

		if (LightType == LIGHT_TYPE_DIRECTIONAL)
		{
			ShadowRayDirection = LightData.Direction;
			ToLight = LightData.Direction * MaxTraceDistance;
		}
		else
		{
			LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, ShadowRayDirection);

			// Skip the light sample that does not contribute anything due to attenuation.
			if (LightMask <= 0.0)
			{
				Lit = 0;
			}
		}

		const bool bHasBackfaceDiffuse = HasBackfaceDiffuse(Payload);
		const bool bBackFace = dot(Payload.GetWorldNormal(), normalize(ToLight)) <= 0;

		// We can skip the light sample pointing backwards if we don't need to compute backface diffuse (e.g. SHADINGMODELID_TWOSIDED_FOLIAGE)
		if (bBackFace && !bHasBackfaceDiffuse)
		{
			Lit = 0;
		}

		if (WaveActiveAllTrue(Lit == 0))
		{
			continue;
		}

		float ShadowRayLength = 0.0;
		if (ShadowsType == RAY_TRACING_SHADOWS_TYPE_SOFT)
		{
			// TODO: Make a permutation for this case? It adds pressure on the live state when not used
			float4 Result = SampleAreaLightDirection(LightData, TranslatedWorldPosition, Payload.GetWorldNormal(), LightType, RandSequence);
			ShadowRayDirection = Result.xyz;
			ShadowRayLength = Result.w;
		}
		else
		{
			// hard shadow or no shadow -- just trace toward light center
			ShadowRayLength = length(ToLight);
		}
		if (ShadowRayLength == 0)
		{
			Lit = 0;
		}

		const bool ApplyShadow = (Lit && ShadowsType != RAY_TRACING_SHADOWS_TYPE_OFF);

		FRayDesc ShadowRay;
		ShadowRay.Origin = TranslatedWorldPosition;
		ShadowRay.Direction = ShadowRayDirection;
		ShadowRay.TMin = 1e-4f;
		ShadowRay.TMax = (ApplyShadow) ? ShadowRayLength : ShadowRay.TMin;

		// Always bias towards the light
		ApplyPositionBias(ShadowRay, bBackFace ? -Payload.GetWorldNormal() : Payload.GetWorldNormal(), ShadowMaxNormalBias);

		uint RayFlags = RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH	// Terminate immediatly after detecting a hit
		                | (ShadowsTranslucencyType == RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_OPAQUE ? RAY_FLAG_FORCE_OPAQUE : 0) // Optionally skip Any Hit Shader
						| RAY_FLAG_SKIP_CLOSEST_HIT_SHADER;			// Skip Closest Hit shader, only run Any Hit shader
		const uint TranslucencyMask = (ShadowsTranslucencyType == RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_FRACTIONAL_VISIBILITY ? RAY_TRACING_MASK_TRANSLUCENT_SHADOW : 0);
		const uint InstanceInclusionMask = (ApplyShadow) ? (RAY_TRACING_MASK_SHADOW | TranslucencyMask) : 0;

		Payload.SetShadowRay();
		Payload.SetMinimalPayloadMode();
		Payload.SetShadowVisibility(1.0);
		Payload.HitT = 0.0;

#if !ENABLE_TWO_SIDED_GEOMETRY
		RayFlags |= RAY_FLAG_CULL_FRONT_FACING_TRIANGLES;
#endif // !ENABLE_TWO_SIDED_GEOMETRY

		// Light index is packed into IndirectIrradiance as well, so is covered by the save/restore of IrradianceColor above
		Payload.SetLightIndex(LightIndex);

#if ENABLE_FAR_FIELD_TRACING
		bool bMissFarField = true;
		{
			// Rebase origin for far-field rays
			FRayDesc FarFieldShadowRay = ShadowRay;

			FDefaultPayload DefaultPayload = (FDefaultPayload)0;
			TraceRay(FarFieldTLAS, RayFlags, InstanceInclusionMask, RAY_TRACING_SHADER_SLOT_SHADOW, RAY_TRACING_NUM_SHADER_SLOTS, RAY_TRACING_MISS_SHADER_SLOT_DEFAULT, FarFieldShadowRay.GetNativeDesc(), DefaultPayload);

			bMissFarField = DefaultPayload.IsMiss();
		}

		// use the bound miss shader lighting evaluation if lighting, else nothing
		if (bMissFarField)
#endif // ENABLE_FAR_FIELD_TRACING
		{
			uint MissShaderIndex = Lit ? LightMissShaderIndex : RAY_TRACING_MISS_SHADER_SLOT_DEFAULT;
			TraceShadowRayMissShaderLighting(ShadowRay, RayFlags, InstanceInclusionMask, TLAS, MissShaderIndex, Payload);
		}
	}

	DirectLighting = Payload.GetRadiance();
	Payload.SetRadiance(OldRadiance);
	// Restore aliased fields
#if SUBSTRATE_GBUFFER_FORMAT==1
	Payload.SceneInstanceIndex = OldSceneInstanceIndex;
	Payload.TransmittancePreCoverage = OldTransmittancePreCoverage;
	Payload.Flags = OldFlags;
#else
	Payload.FlagsAndMipBias = OldFlags;
	Payload.PackedIndirectIrradiance[0] = OldIndirectIrradiance.x;  // Set back indirect irradiance (aliased with ShadowVisibility and LightIndex)
	Payload.PackedIndirectIrradiance[1] = OldIndirectIrradiance.y;
#endif
	Payload.PackedRayCone = OldRayCone;
	return DirectLighting;
}

void ComputeBottomLayerMaterialProperties(FRayDesc Ray, inout FMaterialClosestHitPayload Payload)
{
	// #dxr_todo: Remove me
}

void AccumulateResults(
	inout FPackedMaterialClosestHitPayload Payload,
	in float3 TranslatedWorldPosition,
	in float3 ViewDirection,
	in RaytracingAccelerationStructure TLAS,
	in RaytracingAccelerationStructure FarFieldTLAS,
	inout FRandomSequence RandSequence,
	in uint2 PixelCoord,
	in float ShadowMaxNormalBias,
	float MaxTraceDistance,
	in uint ShadowsType,
	in uint ShadowsTranslucencyType,
	in uint ShouldDoDirectLighting,
	in uint ShouldDoEmissiveAndIndirectLighting,
	in bool bRayTraceSkyLightContribution,
	in bool bDecoupleSampleGeneration,
	inout FRayCone RayCone,
	inout float3 Radiance)
{
	if (Payload.IsMiss())
	{
		return;
	}

	float3 DirectLighting = 0;
		
	if (ShouldDoDirectLighting && Payload.IsValid())
	{
		// Save and restore original payload HitT, as it's modified during shadow ray tracing
		float OldHitT = Payload.HitT;

		DirectLighting = ComputeDirectLighting(TranslatedWorldPosition, ViewDirection, RayCone, TLAS, FarFieldTLAS, Payload, RandSequence, ShadowsType, ShadowsTranslucencyType, ShadowMaxNormalBias, MaxTraceDistance);

		Payload.HitT = OldHitT;
	}

	// Transform NaNs to black, transform negative colors to black.
	DirectLighting = -min(-DirectLighting, float3(0, 0, 0));
	Radiance += DirectLighting;

	if (ShouldDoEmissiveAndIndirectLighting)
	{
		// Emissive & indirect contribution
		Radiance += Payload.GetRadiance() * Payload.GetOpacity();

		// Indirect contribution
		const float3 IndirectLighting = ComputeIndirectLighting(
			TranslatedWorldPosition,
			ViewDirection,
			TLAS,
			PixelCoord,
			Payload,
			bRayTraceSkyLightContribution,
			bDecoupleSampleGeneration);

		Radiance += IndirectLighting;
	}
}

FMaterialClosestHitPayload TraceRayAndAccumulateResults(
	in FRayDesc Ray,
	in RaytracingAccelerationStructure TLAS,
	in RaytracingAccelerationStructure FarFieldTLAS,
	in uint RayFlags,
	in uint InstanceInclusionMask,
	inout FRandomSequence RandSequence,
	in uint2 PixelCoord,
	in float ShadowMaxNormalBias,
	float MaxTraceDistance,
	in uint ShadowsType,
	in uint ShadowsTranslucencyType,
	in uint ShouldDoDirectLighting,
	in uint ShouldDoEmissiveAndIndirectLighting,
	in bool bRayTraceSkyLightContribution,
	in bool bDecoupleSampleGeneration,
	in bool bIsCameraRay,
	inout FRayCone RayCone,
	in bool bEnableSkyLightContribution,
	inout float3 Radiance)
{
	if (bRayTraceSkyLightContribution)
	{
		// Disable precomputed sky light contribution from hit shaders when ray tracing sky light contribution
		bEnableSkyLightContribution = false;
	}

	FPackedMaterialClosestHitPayload Payload = (FPackedMaterialClosestHitPayload)0;
	TraceMaterialRayPacked(
		Payload,
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		Ray,
		RayCone,
		bEnableSkyLightContribution,
		bIsCameraRay);

	float3 TranslatedWorldPosition = Ray.Origin + Ray.Direction * Payload.HitT;
	float3 ViewDirection = Ray.Direction;

	AccumulateResults(
		Payload,
		TranslatedWorldPosition,
		ViewDirection,
		TLAS,
		FarFieldTLAS,
		RandSequence,
		PixelCoord,
		ShadowMaxNormalBias,
		MaxTraceDistance,
		ShadowsType,
		ShadowsTranslucencyType,
		ShouldDoDirectLighting,
		ShouldDoEmissiveAndIndirectLighting,
		bRayTraceSkyLightContribution,
		bDecoupleSampleGeneration,
		RayCone,
		Radiance);

	return UnpackRayTracingPayload(Payload, Ray);
}