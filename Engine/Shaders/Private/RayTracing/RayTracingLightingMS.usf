// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Shared/RayTracingTypes.h"
#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS		0
#define USE_SOURCE_TEXTURE			1
#define USE_SOURCE_TEXTURE_ARRAY	1

#define PreIntegratedGF				ReflectionStruct.PreIntegratedGF			
#define PreIntegratedGFSampler		ReflectionStruct.PreIntegratedGFSampler	

// Ensure that SSS material don't override their albedo to compute only the incident irradiance
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
#define MATERIAL_FULLY_ROUGH 0
#define SUBSTRATE_COMPLEXSPECIALPATH 0
#define SUBSTRATE_GLINTS_ALLOWED 0
#define SUBSTRATE_DIFFUSE_COLOR_BOOST 1

#ifndef SUPPORT_LIGHT_FUNCTION
#define SUPPORT_LIGHT_FUNCTION 0
#endif

#include "../SceneTextureParameters.ush"
#include "RayTracingCommon.ush"
#include "../Lumen/LumenDiffuseColorBoost.ush"

//////////////////////////////////////////////////////////////////////////////////////////
// Include continuous

#if SUBSTRATE_ENABLED
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateEvaluation.ush"
#endif

#include "RayTracingDeferredShadingCommon.ush"
#include "RayTracingLightingCommon.ush" 
#if SUBSTRATE_ENABLED
#include "../Substrate/SubstrateDeferredLighting.ush"
#endif

#if SUPPORT_LIGHT_FUNCTION
#define CameraRelativeLightPosition  RaytracingLightFunctionParameters.CameraRelativeLightPosition
#include "/Engine/Generated/Material.ush"

// TODO: if we could just bind loose parameters, we could avoid this workaround for getting the shader parameters needed by the routines in LightFunctionCommon.ush
#define LightFunctionParameters  RaytracingLightFunctionParameters.LightFunctionParameters
#define LightFunctionParameters2 RaytracingLightFunctionParameters.LightFunctionParameters2
#define LightFunctionTranslatedWorldToLight RaytracingLightFunctionParameters.LightFunctionTranslatedWorldToLight

#include "../LightFunctionCommon.ush"
#include "RayTracingLightFunctionCommon.ush"
#endif

RAY_TRACING_ENTRY_MISS(RayTracingLightingMS, FPackedMaterialClosestHitPayload, PackedPayload)
{
	FRayDesc DummyRay = (FRayDesc)0;

	ResolvedView = ResolveView();

	uint LightIndex = PackedPayload.GetLightIndex();
	float3 CameraVector = PackedPayload.GetRayDirection();// IndirectIrradiance field is used to pass through the camera vector
	FMaterialClosestHitPayload Payload = UnpackRayTracingPayload(PackedPayload, DummyRay);
	float3 V = -CameraVector;
	// reset the data that is aliased with shadow visibility
#if SUBSTRATE_GBUFFER_FORMAT==1
	Payload.TransmittancePreCoverage = 1; 
#else
	Payload.IndirectIrradiance = 0; 
#endif

	uint LightType = 0;
	FDeferredLightData LightData = GetRayTracingDeferredLightData(LightIndex, PackedPayload.IsIndirectLightingRay(), LightType);

	float4 LightAttenuation = 1.0f;

	float3 TranslatedWorldPosition = TranslatedWorldRayOrigin(); // Shadow ray is traced from the shaded point

	float LightProfileMultiplier = ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);

	const float Dither = 0.5f;
	const uint2 SVPos = uint2(0, 0);

	// LWC_TODO - lighting wants translated world space
#if SUBSTRATE_GBUFFER_FORMAT==1
	float3 L = LightData.Direction;	// Already normalized
	float3 ToLight = L;
	
	float LightMask = 1;
	if (LightData.bRadialLight)
	{
		LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
	}

	float3 LightContribution = 0;
	if (LightMask > 0)
	{
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(0,0,0);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(PackedPayload.SubstrateData, SubstrateAddressing, PackedPayload.SubstrateData);

		FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters	= GetInitialisedSubstrateShadowTermInputParameters();
		SubstrateShadowTermInputParameters.bEvaluateShadowTerm					= true;
		SubstrateShadowTermInputParameters.SceneDepth							= 1.0f; // Do not use depth
		SubstrateShadowTermInputParameters.PrecomputedShadowFactors				= 1.0f; // Do not use precomputed factors
		SubstrateShadowTermInputParameters.TranslatedWorldPosition				= TranslatedWorldPosition;
		SubstrateShadowTermInputParameters.LightAttenuation						= LightAttenuation;
		SubstrateShadowTermInputParameters.Dither								= Dither;

		// RayTracing + Lighting + Substrate is too expensive (as in uses too many registers) to be supported on some platforms
		FSubstrateDeferredLighting Lighting = SubstrateDeferredLighting(
			LightData,
			V,
			L,
			ToLight,
			LightMask,
			SubstrateShadowTermInputParameters,
			PackedPayload.SubstrateData,
			SubstrateAddressing,
			SubstratePixelHeader);
		LightContribution = Lighting.SceneColor.xyz;
	}
#else
	// LWC_TODO - lighting wants translated world space
	float SurfaceShadow = 1.0f;
	FGBufferData GBufferData = GetGBufferDataFromPayload(Payload);
	if (PackedPayload.IsIndirectLightingRay())
	{
		GBufferData.DiffuseColor = ApplyDiffuseColorBoost(GBufferData.DiffuseColor, LumenHardwareRayTracingUniformBuffer.DiffuseColorBoost);
	}
	float3 LightContribution = GetDynamicLighting(TranslatedWorldPosition, CameraVector, GBufferData, 1.0f, LightData, LightAttenuation, Dither, SVPos, SurfaceShadow).xyz;
#endif // SUBSTRATE_ENABLED

#if SUPPORT_LIGHT_FUNCTION
	float3 LightFunctionMult = GetRayTracingLightFunction(TranslatedWorldPosition);
	LightProfileMultiplier *= dot(LightFunctionMult, .3333f); // convert to greyscale (for compatibility with the appearance in the primary view)
#endif

	// Apply shadow transparency factor
	LightContribution *= PackedPayload.GetShadowVisibility();

	float3 AccumulatedRadiance = Payload.Radiance + LightContribution * LightProfileMultiplier;

	PackedPayload.SetRadiance(AccumulatedRadiance);
}