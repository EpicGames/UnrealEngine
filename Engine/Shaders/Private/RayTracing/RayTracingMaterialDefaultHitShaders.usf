// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "RayTracingCommon.ush"

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_RAYTRACING_MATERIAL)
RAY_TRACING_ENTRY_CLOSEST_HIT(OpaqueShadowCHS,
	FPackedMaterialClosestHitPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	PackedPayload.HitT = RayTCurrent();
}

RAY_TRACING_ENTRY_CLOSEST_HIT(HiddenMaterialCHS,
	FPackedMaterialClosestHitPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
}

RAY_TRACING_ENTRY_ANY_HIT(HiddenMaterialAHS,
	FPackedMaterialClosestHitPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	IgnoreHit();
}
#endif

#if IS_PAYLOAD_ENABLED(RT_PAYLOAD_TYPE_DECALS)
RAY_TRACING_ENTRY_CALLABLE(DefaultCallableShader,
	FDecalShaderPayload, Params)
{
	// This shader serves only has a placeholder in RTPSO
	// If RTPSO doesn't include a required shader, a NULL SBT entry should be used.
}
#endif
