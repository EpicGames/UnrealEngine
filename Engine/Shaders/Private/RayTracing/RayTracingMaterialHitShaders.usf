// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define ENABLE_SKY_LIGHT 1      
#define NEEDS_LIGHTMAP_COORDINATE (HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP)
#ifdef NEEDS_VERTEX_FACTORY_INTERPOLATION
	#undef NEEDS_VERTEX_FACTORY_INTERPOLATION
#endif
// Needed for VertexFactoryInterpolate to interpolate attributes from vertices to hit point
#define NEEDS_VERTEX_FACTORY_INTERPOLATION 1

// This should be good enough for ray tracing and avoids having to bind an extra buffer
#define EYE_ADAPTATION_PREV_FRAME_EXPOSURE 1

#if SUBSTRATE_ENABLED
#define SUBSTRATE_INLINE_SHADING 1
#define SUBSTRATE_USES_FORCED_COMPLEXITY 0
#endif

// Force simplified RT shader to not use complex special path
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#ifdef SUBSTRATE_COMPLEXSPECIALPATH
#undef SUBSTRATE_COMPLEXSPECIALPATH
#endif
#define SUBSTRATE_COMPLEXSPECIALPATH 0
#endif

#include "/Engine/Private/Common.ush"
#include "RayTracingCommon.ush"
#include "RayTracingHitGroupCommon.ush"
#include "RayTracingShaderUtils.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingFresnel.ush"

#include "/Engine/Generated/Material.ush"   
#include "/Engine/Generated/VertexFactory.ush"

#include "RayTracingCalcInterpolants.ush" // must be included after VertexFactory.ush
#include "/Engine/Private/ShadingCommon.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/SHCommon.ush"
#include "/Engine/Private/ReflectionEnvironmentShared.ush"
#include "/Engine/Private/VirtualTextureCommon.ush" 
#include "/Engine/Private/LightmapCommon.ush"
#include "/Engine/Private/Lumen/LumenHardwareRayTracingPayloadCommon.ush"

#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#include "/Engine/Private/Substrate/SubstrateExport.ush"

/** Computes sky diffuse lighting, including precomputed shadowing. */
void GetSkyLighting(
	FMaterialPixelParameters MaterialParameters,
	VTPageTableResult LightmapVTPageTableResult,
	float3 WorldNormal,
	LightmapUVType LightmapUV,
	uint LightmapDataIndex,
	float3 SkyOcclusionUV3D,
	uint ShadingModel,
	bool bEnableSkyLightContribution,
	out float3 OutDiffuseLighting,
	out float3 OutSubsurfaceLighting)
{
	OutDiffuseLighting = float3(0,0,0);
	OutSubsurfaceLighting = float3(0,0,0);

	// TODO:
	// Should apply Lumen Dynamic GI + shadowed Skylight based on IsLumenTranslucencyGIEnabled() here
	// however we are not binding the unnecessary shader parameters in hitgroups at the moment

#if ENABLE_SKY_LIGHT

	// Check if the Sky Light contribution should be ignored
	if (!bEnableSkyLightContribution)
	{
		return;
	}

	float SkyVisibility = 1;
	float GeometryTerm = 1;
	float3 SkyLightingNormal = WorldNormal;
	
	#if HQ_TEXTURE_LIGHTMAP || CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING || PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
		BRANCH
		if (ShouldSkyLightApplyPrecomputedBentNormalShadowing())
		{
			#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
			
				float3 SkyBentNormal = GetVolumetricLightmapSkyBentNormal(SkyOcclusionUV3D);
				SkyVisibility = length(SkyBentNormal);
				float3 NormalizedBentNormal = SkyBentNormal / max(SkyVisibility, .0001f);

			#elif HQ_TEXTURE_LIGHTMAP

				// Bent normal from precomputed texture
				float4 WorldSkyBentNormalAndOcclusion = GetSkyBentNormalAndOcclusion(LightmapVTPageTableResult, ScaleLightmapUV(LightmapUV, float2(1.0f, 2.0f)), LightmapDataIndex, MaterialParameters.SvPosition.xy);
				// Renormalize as vector was quantized and compressed
				float3 NormalizedBentNormal = normalize(WorldSkyBentNormalAndOcclusion.xyz);
				SkyVisibility = WorldSkyBentNormalAndOcclusion.w;

			#elif CACHED_POINT_INDIRECT_LIGHTING || CACHED_VOLUME_INDIRECT_LIGHTING
 
				// Bent normal from the indirect lighting cache - one value for the whole object
				float3 NormalizedBentNormal = IndirectLightingCache.PointSkyBentNormal.xyz;
				SkyVisibility = IndirectLightingCache.PointSkyBentNormal.w;

			#endif

			#if (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE) && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL)
				// NonDirectional lighting can't depend on the normal
				SkyLightingNormal = NormalizedBentNormal;
			#else
				
				// Weight toward the material normal to increase directionality
				float BentNormalWeightFactor = 1 - (1 - SkyVisibility) * (1 - SkyVisibility);

				// We are lerping between the inputs of two lighting scenarios based on occlusion
				// In the mostly unoccluded case, evaluate sky lighting with the material normal, because it has higher detail
				// In the mostly occluded case, evaluate sky lighting with the bent normal, because it is a better representation of the incoming lighting
				// Then treat the lighting evaluated along the bent normal as an area light, so we must apply the lambert term
				SkyLightingNormal = lerp(NormalizedBentNormal, WorldNormal, BentNormalWeightFactor);

				float DotProductFactor = lerp(saturate(dot(NormalizedBentNormal, WorldNormal)), 1, BentNormalWeightFactor);
				// Account for darkening due to the geometry term
				GeometryTerm = DotProductFactor;
			#endif
		}
	#endif
			
	// Compute the preconvolved incoming lighting with the bent normal direction
	float3 DiffuseLookup = GetSkySHDiffuse(SkyLightingNormal) * ResolvedView.SkyLightColor.rgb;

	// Apply AO to the sky diffuse
	OutDiffuseLighting += DiffuseLookup * (SkyVisibility * GeometryTerm);

	#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	if (ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		float3 BackfaceDiffuseLookup = GetSkySHDiffuse(-WorldNormal) * ResolvedView.SkyLightColor.rgb;
		OutSubsurfaceLighting += BackfaceDiffuseLookup * SkyVisibility;
	}
	#endif

#endif
}

#if !SUBSTRATE_ENABLED || SUBSTRATE_GBUFFER_FORMAT==0
// TODO: Consolidate this definition with similar one in PathTracing Material
float GetRefractionIor(FPixelMaterialInputs PixelMaterialInputs)
{
#if MATERIALBLENDING_TRANSLUCENT && (REFRACTION_USE_INDEX_OF_REFRACTION || REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0)
	// Is the material using refraction at all?
#if REFRACTION_USE_INDEX_OF_REFRACTION
	// Did the user plug something into the Refraction port?
	// Return the user's IOR value, or 0 (ensure we don't get anything negative)
	return max(GetMaterialRefractionIOR(GetMaterialRefraction(PixelMaterialInputs)), 0.0);
#else
	// User did not override the default, compute it via the amount of specular
	float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	float  Specular = GetMaterialSpecular(PixelMaterialInputs);
	float3 F0 = ComputeF0(Specular, BaseColor, Metallic);
	return DielectricF0RGBToIor(F0);
#endif

#else
	// Material either not translucent, or does use have IOR refraction enabled
	return 0.0;
#endif
}
#endif




/** Calculates indirect lighting contribution on this object from precomputed data. */
void GetPrecomputedIndirectLightingAndSkyLight(
	FMaterialPixelParameters MaterialParameters, 
	FVertexFactoryInterpolantsVSToPS Interpolants,
	VTPageTableResult LightmapVTPageTableResult,
	float3 DiffuseDir,
	float3 VolumetricLightmapBrickTextureUVs,
	bool bEvaluateBackface,
	bool bEnableSkyLightContribution,
	out float3 OutDiffuseLighting,
	out float3 OutSubsurfaceLighting,
	out float OutIndirectIrradiance)
{
	OutIndirectIrradiance = 0;
	OutDiffuseLighting = float3(0,0,0);
	OutSubsurfaceLighting = 0;
	LightmapUVType SkyOcclusionUV = (LightmapUVType)0;
	uint SkyOcclusionDataIndex = 0u;

#if HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP
		LightmapUVType LightmapUV0, LightmapUV1;
		uint LightmapDataIndex;
		GetLightMapCoordinates(Interpolants, LightmapUV0, LightmapUV1, LightmapDataIndex);
	#if LIGHTMAP_VT_ENABLED
		LightmapVTPageTableResult = LightmapGetVTSampleInfo(LightmapUV0, LightmapDataIndex, MaterialParameters.SvPosition.xy);
	#endif
#endif

	#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
		FThreeBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH3(VolumetricLightmapBrickTextureUVs);

		// Diffuse convolution
		FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(DiffuseDir, 1);
		OutDiffuseLighting = max(float3(0,0,0), DotSH3(IrradianceSH, DiffuseTransferSH)) / PI;

		#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
		if (bEvaluateBackface)
		{
			FThreeBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH3(-DiffuseDir, 1);
			OutSubsurfaceLighting += max(float3(0,0,0), DotSH3(IrradianceSH, SubsurfaceTransferSH)) / PI;
		}
		#endif

	// High quality texture lightmaps
	#elif HQ_TEXTURE_LIGHTMAP

		SkyOcclusionUV = LightmapUV0;
		SkyOcclusionDataIndex = LightmapDataIndex;
		GetLightMapColorHQ(LightmapVTPageTableResult, LightmapUV0, LightmapUV1, LightmapDataIndex, DiffuseDir, MaterialParameters.SvPosition.xy, bEvaluateBackface, OutDiffuseLighting, OutSubsurfaceLighting);

	// Low quality texture lightmaps
	#elif LQ_TEXTURE_LIGHTMAP

		GetLightMapColorLQ(LightmapVTPageTableResult, LightmapUV0, LightmapUV1, LightmapDataIndex, DiffuseDir, bEvaluateBackface, OutDiffuseLighting, OutSubsurfaceLighting);

	#endif

	// Apply indirect lighting scale while we have only accumulated lightmaps
	OutDiffuseLighting *= View.PrecomputedIndirectLightingColorScale;
	OutSubsurfaceLighting *= View.PrecomputedIndirectLightingColorScale;

	float3 SkyDiffuseLighting;
	float3 SkySubsurfaceLighting;
	GetSkyLighting(
		MaterialParameters,
		LightmapVTPageTableResult,
		DiffuseDir,
		SkyOcclusionUV,
		SkyOcclusionDataIndex,
		VolumetricLightmapBrickTextureUVs,
		bEvaluateBackface,
		bEnableSkyLightContribution,
		SkyDiffuseLighting,
		SkySubsurfaceLighting);

	OutSubsurfaceLighting += SkySubsurfaceLighting;

	// Sky lighting must contribute to IndirectIrradiance for ReflectionEnvironment lightmap mixing
	OutDiffuseLighting += SkyDiffuseLighting;

	#if HQ_TEXTURE_LIGHTMAP || LQ_TEXTURE_LIGHTMAP || CACHED_VOLUME_INDIRECT_LIGHTING || CACHED_POINT_INDIRECT_LIGHTING || PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
		OutIndirectIrradiance = Luminance(OutDiffuseLighting);
	#endif
}

void GetMaterialPayload( 
	FPixelMaterialInputs PixelMaterialInputs,
	FMaterialPixelParameters MaterialParameters,
	FVertexFactoryInterpolantsVSToPS Interpolants,
	bool bIsEnableSkyLightContribution,
	float Dither,
	inout FMaterialClosestHitPayload Payload)
#if SUBSTRATE_GBUFFER_FORMAT==1
{
  #if MATERIAL_IS_SUBSTRATE
	float3 EmissiveLuminance = 0.0f;
	float TotalCoverage = 1.f;
	float3 TransmittancePreCoverage = 0.0f;
	bool bSubstrateSubsurfaceEnable = false;
	const float3 WorldBentNormal0 = GetWorldBentNormalZero(MaterialParameters);
	const float3 CamVector = MaterialParameters.CameraVector;

	// Initialise a Substrate header with normal in registers
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

  #if SUBSTRATE_OPTIMIZED_UNLIT	
	// Unlit BSDF goes through the SubstrateTree to support weighting operations. Technically, layering and mixing could also be supported.
	float3	UnlitSurfaceLuminancePostCoverage = 0.0f;
	float	UnlitSurfaceCoverage = 0.0f;
	float3	UnlitSurfaceTransmittancePreCoverage = 0.0f;
	float3	UnlitSurfaceNormal = 0.0f;
	SubstratePixelHeader.SubstrateUpdateTreeUnlit(
		uint2(MaterialParameters.SvPosition.xy),
		CamVector,
		SubstrateData,
		UnlitSurfaceLuminancePostCoverage,
		UnlitSurfaceCoverage,
		UnlitSurfaceTransmittancePreCoverage,
		UnlitSurfaceNormal);
	EmissiveLuminance = UnlitSurfaceLuminancePostCoverage;

	#if !SUBSTRATE_OPAQUE_MATERIAL
	TotalCoverage = UnlitSurfaceCoverage;
	TransmittancePreCoverage = UnlitSurfaceTransmittancePreCoverage;
	#endif

  #else // SUBSTRATE_OPTIMIZED_UNLIT
	SubstratePixelHeader.IrradianceAO.MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
	SubstratePixelHeader.SetCastContactShadow(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW);
	SubstratePixelHeader.SetDynamicIndirectShadowCasterRepresentation(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION);

	FSubstrateSubsurfaceData SSSData = (FSubstrateSubsurfaceData)0;
	FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
	FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();

	// Generate the Substrate material data to write out
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(0,0,0);

	// Compute TotalCoverage for translucent material
	#if MATERIALBLENDING_ANY_TRANSLUCENT
	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		SubstratePixelHeader.SubstrateUpdateTree(
			SubstrateData, 
			CamVector,
			Settings, 
			TotalCoverage, 
			TransmittancePreCoverage);

	}
	#endif

	Payload.SubstrateData = InitialiseRWSubstrateMaterialContainer();
	PackSubstrateOut(
		Payload.SubstrateData,
		Dither,
		Settings,
		SubstrateAddressing,
		SubstratePixelHeader, SubstrateData, 
		CamVector, 
		WorldBentNormal0,
		bSubstrateSubsurfaceEnable, 
		EmissiveLuminance,
		SSSData, TopLayerData, OpaqueRoughRefractionData
	);

	// Thin surface tracking
	Payload.bIsThinSurface = SubstratePixelHeader.SubstrateGetBSDFType() == SUBSTRATE_BSDF_TYPE_SLAB && BSDF_GETISTHIN(SubstrateData.InlinedBSDF);

	// Sky lighting and lightmap
	Payload.IndirectIrradiance = 0;
	if (bIsEnableSkyLightContribution)
	{
		#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[0]

		float3 VolumetricLightmapBrickTextureUVs;
		#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
		VolumetricLightmapBrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WSHackToFloat(GetWorldPosition(MaterialParameters))); // RT_LWC_TODO
		#endif

		VTPageTableResult LightmapVTPageTableResult = (VTPageTableResult)0.0f;
	
		FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;
		FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, CamVector);

		// Fetch precomputed lighting
		float3 DiffuseLighting = 0.0f;
		float3 SubsurfaceLighting = 0.0f;
		float  IndirectIrradiance = 0.0f;
		const bool bEvaluateBackface = BSDF_GETTYPE(CurrentBSDF) == SUBSTRATE_BSDF_TYPE_SLAB && BSDF_GETSSSTYPE(CurrentBSDF) == SSS_TYPE_TWO_SIDED_WRAP;

		GetPrecomputedIndirectLightingAndSkyLight(
			MaterialParameters, 
			Interpolants, 
			LightmapVTPageTableResult, 
			SubstrateBSDFContext.N, 
			VolumetricLightmapBrickTextureUVs,
			bEvaluateBackface, 
			bIsEnableSkyLightContribution,
			DiffuseLighting, 
			SubsurfaceLighting, 
			IndirectIrradiance);

		Payload.IndirectIrradiance = DiffuseLighting;
		Payload.SetHasIndirectLighting();

		#undef CurrentBSDF
	}

	Payload.SubstrateData.PackedTopLayerData = SubstratePackTopLayerData(TopLayerData);
  #endif // SUBSTRATE_OPTIMIZED_UNLIT

	Payload.Radiance += EmissiveLuminance;
	Payload.TranslatedWorldPos = DFFastToTranslatedWorld(MaterialParameters.AbsoluteWorldPosition, ResolvedView.PreViewTranslation);

#if MATERIALBLENDING_ANY_TRANSLUCENT && (REFRACTION_USE_INDEX_OF_REFRACTION || REFRACTION_USE_INDEX_OF_REFRACTION_FROM_F0)
	#if REFRACTION_USE_INDEX_OF_REFRACTION
	Payload.IorOverride = max(GetMaterialRefractionIOR(GetMaterialRefraction(PixelMaterialInputs)), 0);
	#else
	// IOR will be derived from specular color
	Payload.IorOverride = -1.0f;
	#endif
#else
	Payload.IorOverride = 0;
#endif

	// Opacity
#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
	TotalCoverage = 1.0f;
	TransmittancePreCoverage = 1.0f - GetMaterialOpacity(PixelMaterialInputs);
#endif
	Payload.Opacity = TotalCoverage;
	Payload.TransmittancePreCoverage = TransmittancePreCoverage;

	#if MATERIALBLENDING_MASKED
	// regular masked mode - binary decision
	if (GetMaterialMask(PixelMaterialInputs) < 0)
	{
		Payload.Opacity = 0;
	}
	#endif // MATERIALBLENDING_MASKED


  #endif // MATERIAL_IS_SUBSTRATE
}
#else // SUBSTRATE_GBUFFER_FORMAT==1
{

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	// Here are in the case SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==0

	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();

	const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
	const float3 V = MaterialParameters.CameraVector;
	FExportResult Export = SubstrateMaterialExportOut(
		Settings,
		SubstratePixelHeader,
		SubstrateData,
		V,
		SurfaceWorldNormal,
		MaterialParameters.WorldPosition_CamRelative,
		0.0f // Curvature
	);

	half3 BaseColor = Export.BaseColor;
	half  Metallic = Export.Metallic;
	half  Specular = Export.Specular;
	half  Roughness = Export.Roughness;
	half  IOR = GetRefractionIor(PixelMaterialInputs);		// SUBSTRATE_TODO Task add in order to converrt that function to be Substrate compatible.
	half3 EmissiveColor = Export.EmissiveLuminance;
	half3 WorldNormal = Export.WorldNormal;
	half  Opacity = Export.Coverage;
	half4 CustomData = Export.CustomData;
	half4 ClothData = Export.CustomData;
	half4 SubsurfaceData = half4(Export.SubsurfaceColor, 0.0);

	half3 WorldTangent = 0.0f;
	half Anisotropy = 0.0f;
#if MATERIAL_USES_ANISOTROPY
	WorldTangent = Export.WorldTangent;
	Anisotropy = Export.Anisotropy;
#endif

	Payload.ShadingModelID = Export.ShadingModelID; // Override the shading model using the one from Substrate
#else 
	// Store the results in local variables and reuse instead of calling the functions multiple times.
	half3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half  Metallic = GetMaterialMetallic(PixelMaterialInputs);
	half  Specular = GetMaterialSpecular(PixelMaterialInputs);
	half  Roughness = GetMaterialRoughness(PixelMaterialInputs);
	half  IOR = GetRefractionIor(PixelMaterialInputs);
	half3 EmissiveColor = GetMaterialEmissive(PixelMaterialInputs);
	half3 WorldNormal = normalize(MaterialParameters.WorldNormal);
	half  Opacity = GetMaterialOpacity(PixelMaterialInputs);
	half4 CustomData = half4(GetMaterialCustomData0(PixelMaterialInputs), GetMaterialCustomData1(PixelMaterialInputs), 0, 0);
	half4 SubsurfaceData = GetMaterialSubsurfaceData(PixelMaterialInputs);
	half4 ClothData = float4(SubsurfaceData.rgb, CustomData.x);

	half3 WorldTangent = 0.0f;
	half  Anisotropy = 0.0f;
#if MATERIAL_USES_ANISOTROPY
	WorldTangent = CalculateAnisotropyTangent(MaterialParameters, PixelMaterialInputs);
	Anisotropy = GetMaterialAnisotropy(PixelMaterialInputs);
#endif
#endif

	/**
	 * Set material attributes for full materials
	 **/
	Payload.TranslatedWorldPos = DFFastToTranslatedWorld(MaterialParameters.AbsoluteWorldPosition, ResolvedView.PreViewTranslation);
	Payload.WorldNormal = WorldNormal;
	Payload.Radiance = EmissiveColor;
	Payload.BaseColor = BaseColor;
	Payload.Specular = Specular;
	Payload.Roughness = Roughness;
	Payload.Metallic = Metallic;
#if MATERIALBLENDING_MODULATE
	Payload.Opacity = 0.0;
#else
	// All other blending modes
	Payload.Opacity = Opacity;
#endif

	Payload.Ior = IOR;
	Payload.CustomData = CustomData;
#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	if (Payload.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		Payload.CustomData = half4(SubsurfaceData.rgb, Opacity);
	}
#endif

#if !SUBSTRATE_ENABLED || SUBSTRATE_GBUFFER_FORMAT == 0	// Not need to do that since the SubstrateExport already handle the CustomData all at once.
#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	#if CLEAR_COAT_BOTTOM_NORMAL
	if (Payload.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		float2 oct2 = UnitVectorToOctahedron(Payload.WorldNormal);

		#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
			#if MATERIAL_TANGENTSPACENORMAL
				float3 tempnormal = normalize(TransformTangentVectorToWorld( MaterialParameters.TangentToWorld, ClearCoatBottomNormal0(MaterialParameters) ));
			#else
				float3 tempnormal =  ClearCoatBottomNormal0(MaterialParameters);
			#endif

			float2 oct1 = UnitVectorToOctahedron(tempnormal);
			float2 oct3 = ( (oct1 - oct2) *  0.25 ) + (128.0/255.0);
			Payload.CustomData.a = oct3.x;
			Payload.CustomData.z = oct3.y;
		#else
			Payload.CustomData.a = 128.0/255.0;
			Payload.CustomData.z = 128.0/255.0;
		#endif
	}
	#endif
#endif

#if MATERIAL_SHADINGMODEL_CLOTH
	if (Payload.ShadingModelID == SHADINGMODELID_CLOTH)
	{
		Payload.CustomData = ClothData;
	}
#endif

	// Override custom data if sub-surface material
#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || MATERIAL_SHADINGMODEL_EYE
	if (Payload.ShadingModelID == SHADINGMODELID_SUBSURFACE ||
		Payload.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE ||
		Payload.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE ||
		Payload.ShadingModelID == SHADINGMODELID_EYE)
	{
		Payload.CustomData = SubsurfaceData;
	}
#endif
#endif

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	if (Payload.ShadingModelID == SHADINGMODELID_SINGLELAYERWATER)
	{
		const float3 ScatteringCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput0(MaterialParameters)));	// SUBSTRATE_TODO convert to read from SubstrateData when FrontMaterial is plugged in
		const float3 AbsorptionCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput1(MaterialParameters)));	// SUBSTRATE_TODO rememeber to check SUBSTRATE_INLINE_SINGLELAYERWATER
		const float PhaseG = clamp(DFDemote(GetSingleLayerWaterMaterialOutput2(MaterialParameters).x), -1.0f, 1.0f);

		const float3 WaterExtinction = ScatteringCoeff + AbsorptionCoeff;
		const float3 WaterAlbedo = ScatteringCoeff / max(WaterExtinction, 0.000001);

		Payload.BaseColor.xyz = WaterAlbedo;
		Payload.CustomData.xyz = WaterExtinction;
		Payload.CustomData.w = PhaseG;

		Payload.Ior = DielectricF0ToIor(DielectricSpecularToF0(Specular));
		Payload.BlendingMode = RAY_TRACING_BLEND_MODE_TRANSLUCENT;
	}
#endif

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	if (Payload.ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
	{
		Payload.CustomData.xyz = GetThinTranslucentMaterialOutput0(MaterialParameters);								// SUBSTRATE_TODO convert to read from SubstrateData. We have TransmittancePreCoverage
	}
#endif

#if MATERIAL_USES_ANISOTROPY
	Payload.WorldTangent = WorldTangent;
	Payload.Anisotropy = Anisotropy;
#endif	

	float3 DiffuseIndirectLighting = 0;
	float3 SubsurfaceIndirectLighting;
	float IndirectIrradiance = 0;
	
	float3 VolumetricLightmapBrickTextureUVs;
		
	#if PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING
	VolumetricLightmapBrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WSHackToFloat(GetWorldPosition(MaterialParameters))); // RT_LWC_TODO
	#endif

	VTPageTableResult LightmapVTPageTableResult = (VTPageTableResult)0.0f;

	// Always sample lightmaps without mip biasing, as they are likely packed in an atlas
	// and aggressive bias can cause bleeding between charts.
	float OldGlobalMipBias = GlobalTextureMipBias;
	GlobalTextureMipBias = 0;

	GetPrecomputedIndirectLightingAndSkyLight(
		MaterialParameters,
		Interpolants,
		LightmapVTPageTableResult,
		Payload.WorldNormal,
		VolumetricLightmapBrickTextureUVs,
		GetShadingModelRequiresBackfaceLighting(Payload.ShadingModelID),
		bIsEnableSkyLightContribution,
		DiffuseIndirectLighting,
		SubsurfaceIndirectLighting,
		IndirectIrradiance);

	// Restore global mip bias after lightmaps are sampled.
	GlobalTextureMipBias = OldGlobalMipBias;

	#if MATERIAL_SHADINGMODEL_UNLIT
	if (Payload.ShadingModelID == SHADINGMODELID_UNLIT)
	{
		Payload.Specular = 0;
		Payload.BaseColor = 0;
		Payload.DiffuseColor = 0;
		Payload.SpecularColor = 0;
		Payload.IndirectIrradiance = 0;
	}
	else
	#endif
	{
		Payload.DiffuseColor = BaseColor - BaseColor * Metallic;
		Payload.SpecularColor = ComputeF0(Specular, BaseColor, Metallic);
		Payload.IndirectIrradiance += DiffuseIndirectLighting; // Contains both lightmap & skylight
	}
}
#endif // SUBSTRATE_GBUFFER_FORMAT==1

RAY_TRACING_ENTRY_CLOSEST_HIT(MaterialCHS,
	FPackedMaterialClosestHitPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
	PackedPayload.HitT = RayTCurrent();

#if USE_MATERIAL_CLOSEST_HIT_SHADER
	GlobalTextureMipBias = PackedPayload.GetMipBias();

	ResolvedView = ResolveView();

	FVertexFactoryInterpolantsVSToPS Interpolants;
	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);
	
	FRayCone PropagatedCone = PropagateRayCone(PackedPayload.GetRayCone(), 0 /* surface curvature */, RayTCurrent());
	CalcInterpolants(PropagatedCone, Attributes, Interpolants);

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), GetClusterID(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	float3 GeoNormal = GetGeometryNormalFromTriangleBaseAttributes(PrimitiveIndex(), GetClusterID());
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif

	float Dither = 0.5f;
	bool bIsLumenPayload = PackedPayload.IsLumenPayload();
	if (bIsLumenPayload)
	{
		// GetRandom() must be called before SetSceneInstanceIndex because they share the same storage
		Dither = PackedPayload.GetRandom();

		PackedPayload.SetGeometryNormal(GeoNormal);
		PackedPayload.SetSceneInstanceIndex(GetInstanceUserData());
	}

	if (PackedPayload.IsMinimalPayloadMode())
	{
#if MATERIALBLENDING_TRANSLUCENT
		//For shadow ray that does not ignore translucent materials, the material will be evaluated
		// even with minimal payload mode to query the opacity for translucent shadow.
		if (PackedPayload.IsIgnoreTranslucentMaterials() || !PackedPayload.IsShadowRay())
#endif
		{
			// Minimal payload mode only fills FMinimalPayload::HitT, skipping actual material evaluation.
			// This mode is used when tracing shadow rays against masked geometry.
			// Dynamic branch is used to avoid compiling an extra shader permutation.
			return;
		}
	}

	CurrentPayloadInputFlags = PackedPayload.GetFlags();

	FPixelMaterialInputs PixelMaterialInputs;

	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + WorldRayDirection() * RayTCurrent();
		
		const bool bIsFrontFace = IsFrontFaceHit();

		// #dxr_todo: UE-72130 support world position offset 
		// #if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
		// CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, BasePassInterpolants.PixelPositionExcludingWPO);
		// #else
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
		// #endif
	}

	FMaterialClosestHitPayload Payload = (FMaterialClosestHitPayload)0;

	/**
	 * Set common material attributes for both full and simplified materials
	 **/
	Payload.ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);

	#if MATERIALBLENDING_ALPHACOMPOSITE
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_ALPHA_COMPOSITE;
	#elif MATERIALBLENDING_ALPHAHOLDOUT
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_ALPHA_HOLDOUT;
	#elif MATERIALBLENDING_TRANSLUCENT
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_TRANSLUCENT;
	#elif MATERIALBLENDING_ADDITIVE
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_ADDITIVE;
	#elif MATERIALBLENDING_MODULATE
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_MODULATE;
	#else
		uint MaterialBlendingMode = RAY_TRACING_BLEND_MODE_OPAQUE;
	#endif

	const uint PrimitiveFlags = GetPrimitiveData(MaterialParameters.PrimitiveId).Flags;

	Payload.BlendingMode = MaterialBlendingMode;
	Payload.PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask_FromFlags(PrimitiveFlags);

	Payload.HitT = RayTCurrent();

	if (IsFrontFaceHit())
	{
		Payload.SetFrontFace();
	}

#if MATERIAL_TWOSIDED
	Payload.SetTwoSided();
#endif

	// Fill payload with material data
	GetMaterialPayload(PixelMaterialInputs, MaterialParameters, Interpolants, PackedPayload.IsEnableSkyLightContribution(), Dither, Payload); 

	// After we have gathered the material info, check to see if we are a holdout to see if we need to overwrite the shaded result
#if MATERIALBLENDING_ALPHAHOLDOUT
	const bool bIsHoldout = true;
#else
	const bool bIsHoldout = (PrimitiveFlags & PRIMITIVE_SCENE_DATA_FLAG_HOLDOUT) != 0 && ResolvedView.bPrimitiveAlphaHoldoutEnabled;
#endif
	const bool bIsCameraRay = PackedPayload.IsCameraRay();
	if (bIsHoldout && bIsCameraRay)
	{
#if SUBSTRATE_GBUFFER_FORMAT==1
		Payload.SubstrateData = (FSubstrateRaytracingPayload)0;
		Payload.ShadingModelID = SHADINGMODELID_UNLIT;
		Payload.Radiance = 0.0;
#else
		Payload.ShadingModelID = SHADINGMODELID_UNLIT;
		Payload.Radiance = 0.0;
		Payload.BaseColor = 0.0;
		Payload.Specular = 0.0;
#endif
		Payload.SetHoldout();
	}

	PackedPayload = PackRayTracingPayload(Payload, PropagatedCone);

	// Override packed payload with custom Lumen data
	if (bIsLumenPayload)
	{
		PackedPayload.SetSceneInstanceIndex(GetInstanceUserData());
		PackedPayload.SetGeometryNormal(GeoNormal);
	#if SUBSTRATE_GBUFFER_FORMAT==1
		// RayCone isn't used by Lumen hit-lighting. Reuse Width to pass IOR back to RGS
		FRayCone IorOverride_Unused;
		IorOverride_Unused.Width = Payload.IorOverride;
		IorOverride_Unused.SpreadAngle = 0;
		PackedPayload.SetRayCone(IorOverride_Unused);
	#endif
	}

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	if (bIsCameraRay)
	{
		// Virtual texturing feedback logic (camera rays only for now)
		FinalizeVirtualTextureFeedback(
			MaterialParameters.VirtualTextureFeedback,
			MaterialParameters.SvPosition,
			View.VTFeedbackBuffer
		);
	}
#endif

#endif // USE_MATERIAL_CLOSEST_HIT_SHADER
}

RAY_TRACING_ENTRY_ANY_HIT(MaterialAHS,
	FPackedMaterialClosestHitPayload, PackedPayload,
	FRayTracingIntersectionAttributes, Attributes)
{
#if USE_MATERIAL_ANY_HIT_SHADER

#if MATERIALBLENDING_SOLID
	// Continue traversal by simply returning from this shader
	// Usually this path is inactive because opaque materials don't have AHS bound
	return;
#else
	// All other blending modes _might_ need material evaluation - so we need
	// to generate some code

#if MATERIALBLENDING_MASKED
	// NOTE: Masked mode execute always - regardless of ray flags
	// This ensures that both material rays and shadow rays "see" the same thing
#else
	// For any other blending models, we have a few options:
	if (PackedPayload.IsIgnoreTranslucentMaterials())
	{
		// special mode used by regular RT shadows - translucent materials are skipped, everything else is treated as opaque
		IgnoreHit();
		return;
	}
	else if (!PackedPayload.IsShadowRay())
	{
		// non-opaque blending mode, but we aren't a shadow ray -- continue to CHS
		return;
	}
	else
	{
		// this is a shadow ray and we aren't ignoring translucent materials, so go ahead and run the material logic
	}

	if (PackedPayload.HitT == RayTCurrent())
	{
		// we have processed this intersection already, ignore it
		IgnoreHit();
		return;
	}
#endif  // !MATERIALBLENDING_MASKED

	// If we got here -- we have a reason to evaluate the material -- do the setup 

	GlobalTextureMipBias = PackedPayload.GetMipBias();

	ResolvedView = ResolveView();

	FVertexFactoryInterpolantsVSToPS Interpolants;
	const float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + RayTCurrent() * WorldRayDirection();
	const float4 SvPosition = TranslatedWorldPositionToSvPosition(TranslatedWorldPosition);

	CalcInterpolants(PackedPayload.GetRayCone(), Attributes, Interpolants);

#if VF_SUPPORTS_RAYTRACING_PREPARE_MATERIAL_PIXEL_PARAMETERS
	float3 GeoNormal = 0;
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(TranslatedWorldRayOrigin(), WorldRayDirection(), RayTCurrent(), PrimitiveIndex(), GetClusterID(), Attributes, HitKind(), SvPosition, GeoNormal);
#else
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, SvPosition);
#endif

	CurrentPayloadInputFlags = PackedPayload.GetFlags();

	FPixelMaterialInputs PixelMaterialInputs;

	{
		float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);
		float3 TranslatedWorldPosition = TranslatedWorldRayOrigin() + WorldRayDirection() * RayTCurrent();

		const bool bIsFrontFace = IsFrontFaceHit();

		MaterialParameters.CameraVector = -WorldRayDirection();
		
		// #dxr_todo: UE-72130 support world position offset 
		// #if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS
		// CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, BasePassInterpolants.PixelPositionExcludingWPO);
		// #else
		CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);
		// #endif
	}

	// Now extract the relevant info from PixelMaterialInputs according to the blending mode
#if MATERIALBLENDING_MASKED
	// Regardless of payload flags -- we always apply this 
	if (GetMaterialMask(PixelMaterialInputs) < 0)
	{
		IgnoreHit();
	}
#else // MATERIALBLENDING_MASKED

#if SUBSTRATE_GBUFFER_FORMAT==1
		#if MATERIAL_IS_SUBSTRATE
		float3 EmissiveLuminance = 0.0f;
		float TotalCoverage = 1.f;
		float3 TransmittancePreCoverage = 0.0f;
		bool bSubstrateSubsurfaceEnable = false;
		const float3 WorldBentNormal0 = GetWorldBentNormalZero(MaterialParameters);
		const float3 CamVector = MaterialParameters.CameraVector;

		// Initialise a Substrate header with normal in registers
		FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
		FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

		#if SUBSTRATE_OPTIMIZED_UNLIT
		// Unlit BSDF goes through the SubstrateTree to support weighting operations. Technically, layering and mixing could also be supported.
		float3 UnlitSurfaceLuminancePostCoverage = 0.0f;
		float UnlitSurfaceCoverage = 0.0f;
		float3 UnlitSurfaceTransmittancePreCoverage = 0.0f;
		float3 UnlitSurfaceNormal = 0.0f;
		SubstratePixelHeader.SubstrateUpdateTreeUnlit(
			uint2(MaterialParameters.SvPosition.xy),
			CamVector,
			SubstrateData,
			UnlitSurfaceLuminancePostCoverage,
			UnlitSurfaceCoverage,
			UnlitSurfaceTransmittancePreCoverage,
			UnlitSurfaceNormal);
		EmissiveLuminance = UnlitSurfaceLuminancePostCoverage;

		TotalCoverage = UnlitSurfaceCoverage;
		TransmittancePreCoverage = UnlitSurfaceTransmittancePreCoverage;
		const float3 Transparency = lerp(1, TransmittancePreCoverage, TotalCoverage);

		#else // SUBSTRATE_OPTIMIZED_UNLIT
		SubstratePixelHeader.IrradianceAO.MaterialAO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
		SubstratePixelHeader.SetCastContactShadow(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW);
		SubstratePixelHeader.SetDynamicIndirectShadowCasterRepresentation(GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION);

		FSubstrateSubsurfaceData SSSData = (FSubstrateSubsurfaceData)0;
		FSubstrateTopLayerData TopLayerData = (FSubstrateTopLayerData)0;
		FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;
		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();

		// Generate the Substrate material data to write out
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(0, 0, 0);

		// Compute TotalCoverage for translucent material
		if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
		{
			SubstratePixelHeader.SubstrateUpdateTree(
				SubstrateData,
				CamVector,
				Settings,
				TotalCoverage,
				TransmittancePreCoverage);

		}
		const float3 Transparency = lerp(1, TransmittancePreCoverage, TotalCoverage);

		#endif // SUBSTRATE_OPTIMIZED_UNLIT
		#else // MATERIAL_IS_SUBSTRATE
		// SUBSTRATE_TODO: What materials are in this case?
		const float Transparency = 0;
		#endif // MATERIAL_IS_SUBSTRATE
#else // SUBSTRATE_GBUFFER_FORMAT==0 

#if MATERIALBLENDING_MODULATE
	const float3 Transparency = GetMaterialEmissive(PixelMaterialInputs);
#elif MATERIALBLENDING_ALPHACOMPOSITE
	const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
	const float Transparency = 1 - Opacity;
#elif MATERIALBLENDING_TRANSLUCENT
	const float Opacity = GetMaterialOpacity(PixelMaterialInputs);
#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	float3 Transparency = 1 - Opacity;
	if (Opacity < 1.0)
	{
		float3 Transmission = GetThinTranslucentMaterialOutput0(MaterialParameters);
		float3 V = WorldRayDirection();
		float3 N = normalize(MaterialParameters.WorldNormal);
		float VoN = abs(dot(V, N));
		// compute transmission through the slab (fresnel + absorption)
		float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
		float Metallic = GetMaterialMetallic(PixelMaterialInputs);
		float Specular = GetMaterialSpecular(PixelMaterialInputs);
		float F0 = F0RGBToF0(ComputeF0(Specular, BaseColor, Metallic));
		float Ior = GetRefractionIor(PixelMaterialInputs);
		Transparency *= ComputeThinSlabWeights(Transmission, VoN, Ior, F0).Transmitted;
	}
	// Fade out the Transparency as a function of the SurfaceCoverage.
	const float SurfaceCoverage = GetThinTranslucentMaterialOutput1(MaterialParameters);
	Transparency = lerp(1.0.xxx, Transparency, SurfaceCoverage);
#else // !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	float Transparency = 1 - Opacity;
	uint ShadingModelID = GetMaterialShadingModel(PixelMaterialInputs);
#if MATERIAL_SHADINGMODEL_DEFAULT_LIT && REFRACTION_USE_INDEX_OF_REFRACTION
	if (ShadingModelID == SHADINGMODELID_DEFAULT_LIT)
	{
		float Ior = GetRefractionIor(PixelMaterialInputs);
		if (Transparency > 0 && Ior > 0.0)
		{
			// Material is doing refraction, apply a fake caustic term to represent this
			float3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
			float  Metallic = GetMaterialMetallic(PixelMaterialInputs);
			float  Specular = GetMaterialSpecular(PixelMaterialInputs);
			float  F0 = F0RGBToF0(ComputeF0(Specular, BaseColor, Metallic));
			float3 N = normalize(MaterialParameters.WorldNormal);
			float NoV = abs(dot(WorldRayDirection(), N));
			float Fr = FresnelReflectance(NoV, Ior, F0);
			Transparency *= Pow2(1 - Fr);
		}
	}
#endif  // MATERIAL_SHADINGMODEL_DEFAULT_LIT
#endif // !MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
#endif // MATERIALBLENDING_TRANSLUCENT
#endif // SUBSTRATE_GBUFFER_FORMAT==1

	float3 ShadowVis = PackedPayload.GetShadowVisibility();
	ShadowVis *= Transparency;
	PackedPayload.SetShadowVisibility(ShadowVis);
	PackedPayload.HitT = RayTCurrent();
	// Keep going if we haven't reached full opacity
	if (any(ShadowVis > 0.0))
	{
		IgnoreHit();
	}
#endif // !MATERIALBLENDING_MASKED

#endif // MATERIALBLENDING_SOLID

#endif // USE_MATERIAL_ANY_HIT_SHADER
}
