// Copyright Epic Games, Inc. All Rights Reserved.

// Bump to force a rebuild of this shader (when changing related C++ code). Generate unique value on merge conflict.
#pragma message("UESHADERMETADATA_VERSION 42D4CD2F-D217-4305-81AB-F113F7114047")

// Substrate use view's resources to read & trace shadow ray
#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 1

#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "../DeferredShadingCommon.ush"
#include "../LightShaderParameters.ush"
#include "../SceneTextureParameters.ush"
#include "../ScreenSpaceDenoise/SSDPublic.ush"
#include "../TransmissionCommon.ush"
#include "../HairStrands/HairStrandsVisibilityCommon.ush"
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateSubsurface.ush"
#include "RayTracingCommon.ush"
#include "RayTracingDeferredShadingCommon.ush"
#include "RayTracingDirectionalLight.ush"
#include "RayTracingRectLight.ush"
#include "RayTracingSphereLight.ush"
#include "RayTracingCapsuleLight.ush"
#include "/Engine/Shared/LightDefinitions.h"

#define USE_TRANSLUCENT_SHADOW  !COMPUTESHADER // support translucent shadow path if any-hit shading is available

#define ShadowMaskType_Opaque 0
#define ShadowMaskType_Hair 1

RaytracingAccelerationStructure TLAS;
RWTexture2D<float4> RWOcclusionMaskUAV;
RWTexture2D<float> RWRayDistanceUAV;
RWTexture2D<float4> RWSubPixelOcclusionMaskUAV;

uint LightingChannelMask;
uint SamplesPerPixel;
float NormalBias;
int4 LightScissor;
int2 PixelOffset;
float TraceDistance;
float LODTransitionStart;
float LODTransitionEnd;
float AvoidSelfIntersectionTraceDistance;
uint bTransmissionSamplingDistanceCulling;
uint TransmissionSamplingTechnique;
uint TransmissionMeanFreePathType;
uint RejectionSamplingTrials;
uint bAcceptFirstHit;
uint bTwoSidedGeometry;
uint bTranslucentShadow;
uint MaxTranslucencyHitCount;

#define TransmissionSampling_ConstantTrackingInfiniteDomain 0
#define TransmissionSampling_ConstantTrackingFiniteDomain 1

#define TransmissionMeanFreePathType_ExtinctionScale	0
#define TransmissionMeanFreePathType_MaxMeanFreePath	1

#if USE_HAIR_LIGHTING
#include "../HairStrands/HairStrandsRaytracing.ush"
uint				bUseHairVoxel;
float				HairOcclusionThreshold;
Texture2D<uint>		HairLightChannelMaskTexture;

bool NeedTraceHair(
	in uint2 PixelCoord,
	inout float HairDeviceZ)
{
	const float HairSampleDepth = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelCoord, 0)).x;
	bool bTraceHairRay = false;
	HairDeviceZ = 0;
	if (HairSampleDepth > 0)
	{
		const uint HairLightChannel = HairLightChannelMaskTexture.Load(uint3(PixelCoord, 0));
		HairDeviceZ = HairSampleDepth;
		bTraceHairRay = (HairLightChannel & LightingChannelMask) != 0;
	}

	return bTraceHairRay;
}
#endif

bool GenerateOcclusionRay(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax
)
{
	#if LIGHT_TYPE == LIGHT_TYPE_DIRECTIONAL
	{
		GenerateDirectionalLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax);
	}
	#elif LIGHT_TYPE == LIGHT_TYPE_POINT || LIGHT_TYPE == LIGHT_TYPE_SPOT
	{
		#if LIGHT_TYPE == LIGHT_TYPE_SPOT
			// before generating a shadow ray, make sure we are inside the cone of the light
			float3 LightDirection = normalize(LightParameters.TranslatedWorldPosition - TranslatedWorldPosition);
			float CosAngle = LightParameters.SpotAngles.x;
			if (!(dot(LightDirection, LightParameters.Direction) >= CosAngle))
			{
				// outside the cone of the light, skip all work
				RayOrigin = (float3)0;
				RayDirection = (float3)0;
				RayTMin = 0.0f;
				RayTMax = 0.0f;
				return false;
			}
		#endif
		float RayPdf;
		if (LightParameters.SourceLength > 0.0)
		{
			return GenerateCapsuleLightOcclusionRayWithSolidAngleSampling(
				LightParameters,
				TranslatedWorldPosition, WorldNormal,
				RandSample,
				/* out */ RayOrigin,
				/* out */ RayDirection,
				/* out */ RayTMin,
				/* out */ RayTMax,
				/* out */ RayPdf);
		}
		return GenerateSphereLightOcclusionRayWithSolidAngleSampling(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax,
			/* out */ RayPdf);
	}
	#elif LIGHT_TYPE == LIGHT_TYPE_RECT
	{
		float RayPdf = 0.0;
		return GenerateRectLightOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ RayOrigin,
			/* out */ RayDirection,
			/* out */ RayTMin,
			/* out */ RayTMax,
			/* out */ RayPdf);
	}
	#else
		#error Unknown light type.
	#endif
	return true;
}

float ComputeDiffuseTransmission(
	float3 V, float3 P, float3 N,
	float MeanFreePath, float MaxTransmissionDistance, float Eta,
	FLightShaderParameters LightParameters, float2 LightSample,
	inout FRandomSequence RandSequence, 
	uint RejectionRetryMax, uint2 PixelCoord)
{
	float TransmissionDistance = MaxTransmissionDistance;
	float RcpMeanFreePath = rcp(MeanFreePath);

	// Refract inward and force a scattering event
	float3 Tr = refract(V, N, Eta);
	if (all(Tr) == 0.0)
	{
		return TransmissionDistance;
	}

	// Find bounding sub-surface ray extent
	const uint RayFlags = 0;
	const uint InstanceInclusionMask = RAY_TRACING_MASK_OPAQUE;

	float TMax = TransmissionDistance;
	if (bTransmissionSamplingDistanceCulling)
	{
		FRayDesc RejectionTestRay;
		RejectionTestRay.Origin = P;
		RejectionTestRay.Direction = Tr;
		RejectionTestRay.TMin = 0.01;
		RejectionTestRay.TMax = TMax;

		FMinimalPayload RejectionTestPayload = TraceVisibilityRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			RejectionTestRay);
		TMax = (RejectionTestPayload.IsHit() ? RejectionTestPayload.HitT : RejectionTestRay.TMax);
		
		// Assume the back facing is parallel to the front face.
		// If the first ray marching into the volume does not hit anything, we should still
		// resume for the scattering such that it has a chance to fly out of the volume
		// based on the incident angle and the angle to light 
		/*if (RejectionTestPayload.IsMiss())
		{
			return TransmissionDistance;
		}*/
	}

	float RandSample = RandSequence.Get1D();

	float ScatterDistance = TransmissionDistance;
	float ScatterDistancePdf = 0.0;
	if (TransmissionSamplingTechnique == TransmissionSampling_ConstantTrackingFiniteDomain)
	{
		// Use constant tracking through homogeneous media, but renormalize to force a scatter event within the interval: [0, TMax]
		float NormalizationConstant = 1.0 - exp(-RcpMeanFreePath * TMax);
		ScatterDistance = -log(1.0 - RandSample * NormalizationConstant) * MeanFreePath;
		ScatterDistancePdf = (TMax * RcpMeanFreePath) * exp(-RcpMeanFreePath * ScatterDistance) * rcp(NormalizationConstant);
	}
	else // TransmissionSamplingTechnique = TransmissionSampling_ConstantTrackingInfiniteDomain
	{
		// Use constant tracking through homogeneous media, with rejection sampling to force a scatter event
		ScatterDistance = -log(1 - RandSample) * MeanFreePath;
		ScatterDistancePdf = RcpMeanFreePath * exp(-ScatterDistance * RcpMeanFreePath);

		// Reject scatter distances which penetrate through the medium
		uint RetryCount = 0;
		while (ScatterDistance > TMax && RetryCount < RejectionSamplingTrials)
		{
			RandSample = RandSequence.Get1D();
			ScatterDistance = -log(1 - RandSample) * MeanFreePath;
			ScatterDistancePdf = RcpMeanFreePath * exp(-ScatterDistance * RcpMeanFreePath);
			RetryCount++;
		}
		if (ScatterDistance > TMax)	return 0.0;
	}

	// After sampling by pdf, the first interaction could go out of the range
	// resulting in no transmission. Need to reconsider adding it in by commenting this out?
	ScatterDistance /= ScatterDistancePdf;

	// Avoid self intersection.
	ScatterDistance = max(0.01, ScatterDistance);

	// Build sub-surface scattering ray
	FRayDesc SSSRay;
	SSSRay.Origin = P + Tr * ScatterDistance;
	bool bIsValidRay = GenerateOcclusionRay(LightParameters, SSSRay.Origin, N, LightSample, SSSRay.Origin, SSSRay.Direction, SSSRay.TMin, SSSRay.TMax);

	if (!bIsValidRay)
	{
		return TransmissionDistance;
	}

	// Clip the ray length by the maximum transmission distance
	float LightTMax = SSSRay.TMax;
	SSSRay.TMax = TransmissionDistance - ScatterDistance;

	FMinimalPayload SSSPayload = TraceVisibilityRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		SSSRay);

	if (SSSPayload.IsHit())
	{
		// Confirm that there is not an actual occluder beyond the maximum transmission distance
		SSSRay.TMin = SSSPayload.HitT + 0.01;
		SSSRay.TMax = LightTMax;
		FMinimalPayload VisibilityPayload = TraceVisibilityRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			SSSRay);

		// Take into account hair strands occlusion when computing transmission distance
		#if USE_HAIR_LIGHTING
		if (bUseHairVoxel && VisibilityPayload.IsMiss())
		{
			const float HairHitT = TraverseHair(PixelCoord, RandSequence, SSSRay.Origin, SSSRay.Direction, /*SSSRay.TMin*/-1, VirtualVoxel.Raytracing_ShadowOcclusionThreshold);
			if (HairHitT >= -1)
			{
				VisibilityPayload.HitT = min(HairHitT, SSSRay.TMin);
			}
		}
		#endif

		if (VisibilityPayload.IsMiss())
		{
			TransmissionDistance = ScatterDistance + SSSPayload.HitT;
		}
	}

	return TransmissionDistance;
}

struct FOcclusionResult
{
	float Visibility;
	float SumRayDistance;
	float ClosestRayDistance;
	float TransmissionDistance;
	float HitCount;
	float RayCount;
};

FOcclusionResult InitOcclusionResult()
{
	FOcclusionResult Out;

	Out.Visibility = 0.0;
	Out.SumRayDistance = 0.0;
	Out.ClosestRayDistance = DENOISER_INVALID_HIT_DISTANCE;
	Out.TransmissionDistance = 0.0;
	Out.HitCount = 0.0;
	Out.RayCount = 0.0;

	return Out;
}

float OcclusionToShadow(FOcclusionResult In, uint LocalSamplesPerPixel)
{
	return (LocalSamplesPerPixel > 0) ? In.Visibility / LocalSamplesPerPixel : In.Visibility;
}

FRayDesc Invert(in FRayDesc Ray)
{
	FRayDesc InvertedRay;
	InvertedRay.TMin = Ray.TMin;
	InvertedRay.TMax = Ray.TMax;
#if LIGHT_TYPE == LIGHT_TYPE_DIRECTIONAL
	InvertedRay.TMax = min(InvertedRay.TMax, TraceDistance);
#endif
	InvertedRay.Origin = Ray.Origin + Ray.Direction * InvertedRay.TMax;
	InvertedRay.Direction = -Ray.Direction;

	return InvertedRay;
}

struct FOcclusionShadingParameters
{
	bool bIsValid;
	bool bBackfaceCulling;
	bool bIsHair;
	bool bIsHairStrands;
	bool bIsSubsurface;
	bool bHasSubsurfaceProfile;
};

FOcclusionResult ComputeOcclusion(
	const uint2 PixelCoord,
	const uint RaytracingMask,
	const float DeviceZ,
	float3 WorldNormal,
	const FOcclusionShadingParameters ShadingDesc,
	const FLightShaderParameters LightParameters,
	const FTransmissionProfileParams TransmissionProfileParams,
	const float SubsurfaceSamplingMeanFreePath,
	const uint LocalSamplesPerPixel)
{
	FOcclusionResult Out = InitOcclusionResult();

	const float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDeviceZ(PixelCoord, DeviceZ);

	// For hair shading model, WorldNormal is actually the shading tangent.
	// To generate proper bias, we compute a normal oriented toward the light.
	// Normal clipping is removed from hair since the BSDF is spherical, rather 
	// than hemispherical
	//
	// Note: 
	// Since we don't have a notion of backfacing here, there is no correct way 
	// to avoid self-intersection with hair. Either we push towards the light, 
	// (this creates some issue in triangle facing back the light), or we push 
	// toward the view point (this create issue for transmitted light)
	// It seems having proper transmission is more important, thus the light 
	// version is enabled by default
	if (ShadingDesc.bIsHair || ShadingDesc.bIsHairStrands)
	{
	#if LIGHT_TYPE == LIGHT_TYPE_DIRECTIONAL
		const float3 LightDirection = LightParameters.Direction;
	#else
		const float3 LightDirection = normalize(LightParameters.TranslatedWorldPosition - TranslatedWorldPosition);
	#endif
		WorldNormal = LightDirection;
	}

	bool bApplyNormalCulling = ShadingDesc.bBackfaceCulling;

	// Disable normal culling for RectLights
	if (LIGHT_TYPE == LIGHT_TYPE_RECT)
	{
		bApplyNormalCulling = false;
	}

#if ENABLE_MULTIPLE_SAMPLES_PER_PIXEL
	LOOP for (uint SampleIndex = 0; SampleIndex < LocalSamplesPerPixel; ++SampleIndex)
#else // ENABLE_MULTIPLE_SAMPLES_PER_PIXEL
	do if (LocalSamplesPerPixel > 0)
#endif // ENABLE_MULTIPLE_SAMPLES_PER_PIXEL
	{
		

#if ENABLE_MULTIPLE_SAMPLES_PER_PIXEL
		FRandomSequence RandSequence  = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), SampleIndex, LocalSamplesPerPixel);
#else
		FRandomSequence RandSequence  = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), 0, 1);
#endif

		float2 RandSample = RandSequence.Get2D();

		FRayDesc Ray = (FRayDesc)0;
		bool bIsValidRay = GenerateOcclusionRay(
			LightParameters,
			TranslatedWorldPosition, WorldNormal,
			RandSample,
			/* out */ Ray.Origin,
			/* out */ Ray.Direction,
			/* out */ Ray.TMin,
			/* out */ Ray.TMax);
			

		uint Stencil = SceneStencilTexture.Load(int3(PixelCoord, 0)) STENCIL_COMPONENT_SWIZZLE;
		bool bDitheredLODFadingOut = Stencil & 1;
		
#if	ENABLE_TRANSMISSION
		float NoL = dot(WorldNormal, Ray.Direction);
		if (NoL > 0.0)
		{
			ApplyCameraRelativeDepthBias(Ray, PixelCoord, DeviceZ, WorldNormal, NormalBias + (bDitheredLODFadingOut ? 0.8f : 0.0f));
		}
		else
		{
			ApplyPositionBias(Ray, -WorldNormal, NormalBias);
		}
#else
		ApplyCameraRelativeDepthBias(Ray, PixelCoord, DeviceZ, WorldNormal, NormalBias + (bDitheredLODFadingOut ? 0.8f : 0.0f));
#endif

		BRANCH
		if (!bIsValidRay && (DIM_DENOISER_OUTPUT == 0))
		{
			// The denoiser must still trace invalid rays to get the correct closest-hit distance
			continue;
		}
		else if (bApplyNormalCulling && dot(WorldNormal, Ray.Direction) <= 0.0)
		{
			continue;
		}

		// Attenuation check
		if (Ray.TMax * LightParameters.InvRadius > 1.0)
		{
			continue;
		}

		uint RayFlags = 0;

		// Enable back face culling if not used two-sided shadow casting mode.
		if (bTwoSidedGeometry != 1)
		{
			RayFlags |= RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
		}

	#if USE_HAIR_LIGHTING
		if (ShadingDesc.bIsHairStrands)
		{
			// Denoiser mode 0 doesn't use depth, so take first hit
			RayFlags |= RAY_FLAG_SKIP_CLOSEST_HIT_SHADER | RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH;
		}
	#endif

		uint RayFlagsForOpaque = bAcceptFirstHit != 0 ? RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH : 0;

		FMinimalPayload MinimalPayload;

#if AVOID_SELF_INTERSECTION_TRACE
		{
			FRayDesc EpsilonRay = Ray;
			EpsilonRay.TMax = AvoidSelfIntersectionTraceDistance;

			if (Ray.TMax > Ray.TMin)
			{
				MinimalPayload = TraceVisibilityRay(
					TLAS,
					RayFlags | RayFlagsForOpaque | RAY_FLAG_CULL_BACK_FACING_TRIANGLES,
					RaytracingMask,
					EpsilonRay);
			}
		}

		if (!MinimalPayload.IsHit())
		{
			Ray.TMin = max(Ray.TMin, AvoidSelfIntersectionTraceDistance);

			MinimalPayload = TraceVisibilityRay(
				TLAS,
				RayFlags | RayFlagsForOpaque,
				RaytracingMask,
				Ray);
		}
#else
		MinimalPayload = TraceVisibilityRay(
			TLAS,
			RayFlags | RayFlagsForOpaque,
			RaytracingMask,
			Ray);
#endif

		#if USE_HAIR_LIGHTING
		if (!ShadingDesc.bIsHairStrands && bUseHairVoxel)
		{
			MinimalPayload.HitT = TraverseHair(PixelCoord, RandSequence, Ray.Origin, Ray.Direction, MinimalPayload.HitT, VirtualVoxel.Raytracing_ShadowOcclusionThreshold);
		}
		#endif

		Out.RayCount += 1.0;
		if (MinimalPayload.IsHit())
		{
			float HitT = MinimalPayload.HitT;

			Out.ClosestRayDistance =
				(Out.ClosestRayDistance == DENOISER_INVALID_HIT_DISTANCE) ||
				(HitT < Out.ClosestRayDistance) ? HitT : Out.ClosestRayDistance;
			Out.SumRayDistance += HitT;
			Out.HitCount += 1.0;

			if (ShadingDesc.bIsSubsurface || ShadingDesc.bIsHair)
			{
				// Reverse the ray to support sub-surface casts
				FRayDesc InvertedRay = Invert(Ray);
				FMinimalPayload SubsurfacePayload = TraceVisibilityRay(
					TLAS,
					RayFlags,
					RaytracingMask,
					InvertedRay);

				if (SubsurfacePayload.IsHit())
				{

					float3 HitPosition = InvertedRay.Origin + InvertedRay.Direction * SubsurfacePayload.HitT;
					float Opacity = TransmissionProfileParams.ExtinctionScale;
					float Density = -.05f * log(1 - min(Opacity, .999f));
					if (ShadingDesc.bIsHair)
					{
						Opacity = 1;
						Density = 1;
					}
					float ExtinctionCoefficient = Density;
					float Thickness = length(HitPosition - TranslatedWorldPosition);
					float Transmission = saturate(exp(-ExtinctionCoefficient * Thickness));

					if (Transmission == 1.0)
					{
						Out.ClosestRayDistance = (Out.ClosestRayDistance == DENOISER_INVALID_HIT_DISTANCE) ? DENOISER_MISS_HIT_DISTANCE : Out.ClosestRayDistance;
						Out.SumRayDistance -= HitT;
						Out.HitCount -= 1.0;
						Out.TransmissionDistance += 1.0;
						Out.Visibility += 1.0;
					}

					Out.TransmissionDistance += Transmission;
				}
			}
		}
		else
		{
#if USE_TRANSLUCENT_SHADOW
			bool bHasClosestEffectiveTranslucentHit = false;

			if (bTranslucentShadow)
			{
				// When there is no intersection, test for translucent material hit if translucent shadow
				// is supported. If there is a hit, record the accumulated opacity.
				FTranslucentMinimalPayload TranslucentMinimalPayload = TraceTranslucentVisibilityRay(
					TLAS,
					RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER,
					RAY_TRACING_MASK_TRANSLUCENT_SHADOW,
					Ray);
				Out.ClosestRayDistance = (Out.ClosestRayDistance == DENOISER_INVALID_HIT_DISTANCE) ? DENOISER_MISS_HIT_DISTANCE : Out.ClosestRayDistance;
				Out.TransmissionDistance += 1.0;
				Out.Visibility += Luminance(TranslucentMinimalPayload.ShadowVisibility);
			}
			else
#endif
			{
				Out.ClosestRayDistance = (Out.ClosestRayDistance == DENOISER_INVALID_HIT_DISTANCE) ? DENOISER_MISS_HIT_DISTANCE : Out.ClosestRayDistance;
				Out.TransmissionDistance += 1.0;
				Out.Visibility += 1.0;
			}
		}
	}
#if !ENABLE_MULTIPLE_SAMPLES_PER_PIXEL
	while (0);
#endif // ENABLE_MULTIPLE_SAMPLES_PER_PIXEL

	if (ENABLE_TRANSMISSION && LocalSamplesPerPixel > 0 && ShadingDesc.bHasSubsurfaceProfile)
	{
		FRandomSequence RandSequence = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), 0, 1);

		// Fix light sampling to occur at the light center for all transmission computations..
		float2 LightSample = 0.5;
		const float3 TranslatedWorldViewOrigin = DFFastToTranslatedWorld(PrimaryView.WorldViewOrigin, PrimaryView.PreViewTranslation);
		const float3 V = normalize(TranslatedWorldPosition - TranslatedWorldViewOrigin);

		// Apply sqrt to match the behavior of the rasterizer closer visually when ExtinctionScale <=1.
		const float ExtinctionScale = TransmissionProfileParams.ExtinctionScale;
		
		// Decrease ExtinctionScale should increase the max transmission distance. Keep rasterizer and raytracing shadow behave similar for transmission.
		const float MaxTransmissionDistance = SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE / ExtinctionScale; 
		const float Eta = TransmissionProfileParams.OneOverIOR;
		const uint RejectionRetryMax = LocalSamplesPerPixel - 1;
		Out.TransmissionDistance = ComputeDiffuseTransmission(V, TranslatedWorldPosition, WorldNormal, SubsurfaceSamplingMeanFreePath,
			MaxTransmissionDistance, Eta, LightParameters, LightSample, RandSequence, RejectionRetryMax, PixelCoord);

		// Overload closest-hit distance to express occlusion distance when facing the light or transmission distance when back-facing
		if (Out.TransmissionDistance < MaxTransmissionDistance)
		{
			Out.ClosestRayDistance = max(Out.ClosestRayDistance, Out.TransmissionDistance);
		}

		// Respond to the normal scale
		const float NormalScale = TransmissionProfileParams.NormalScale * 0.5;
		Out.TransmissionDistance += NormalScale;

		// Conversion to raster model: 1.0 - (distance / SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE).
		Out.TransmissionDistance = saturate(EncodeOpticalDepthToShadowMask(Out.TransmissionDistance * ExtinctionScale));
	}
	else if (ShadingDesc.bIsSubsurface || ShadingDesc.bIsHair)
	{
		float Depth = GetDistanceToCameraFromViewVector(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);
		float Range = LODTransitionEnd - LODTransitionStart;
		if (Depth > LODTransitionStart && Range > 0.0)
		{
			float Alpha = saturate((Depth - LODTransitionStart) / Range);
			Out.Visibility = lerp(Out.Visibility, Out.TransmissionDistance, Alpha);
		}
		Out.TransmissionDistance = (LocalSamplesPerPixel > 0) ? Out.TransmissionDistance / LocalSamplesPerPixel : Out.TransmissionDistance;
	}
	else // Eye/Foliage (SHADINGMODELID_EYE, SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		Out.TransmissionDistance = (LocalSamplesPerPixel > 0) ? Out.Visibility / LocalSamplesPerPixel : Out.Visibility;
	}
	return Out;
}

#if SUBSTRATE_GBUFFER_FORMAT==0
float DecodeSamplingSSSProfileRadius(FGBufferData GBufferData)
{
	uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(GBufferData);

	// Burley Sampling parameterization
	float3 SurfaceAlbedo = View.SSProfilesTexture.Load(int3(BSSS_SURFACEALBEDO_OFFSET, SubsurfaceProfileInt, 0)).www;
	float3 DiffuseMeanFreePath = DecodeDiffuseMeanFreePath(View.SSProfilesTexture.Load(int3(BSSS_DMFP_OFFSET, SubsurfaceProfileInt, 0))).www;
	float WorldUnitScale = DecodeWorldUnitScale(View.SSProfilesTexture.Load(int3(SSSS_TINT_SCALE_OFFSET, SubsurfaceProfileInt, 0)).a) * BURLEY_CM_2_MM;
	float Opacity = UseSubsurfaceProfile(GBufferData.ShadingModelID) ? GBufferData.CustomData.a : 0.0f;

	float SSSRadius = GetMFPFromDMFPApprox(SurfaceAlbedo, SurfaceAlbedo, Opacity * WorldUnitScale * DiffuseMeanFreePath).z;

	return SSSRadius * BURLEY_MM_2_CM;
}
#endif

RAY_TRACING_ENTRY_RAYGEN_OR_INLINE(Occlusion)
{
	uint2 PixelCoord = DispatchThreadIndex.xy + View.ViewRectMin.xy + PixelOffset;

	FOcclusionResult Occlusion = InitOcclusionResult();
	FOcclusionResult HairOcclusion = InitOcclusionResult();

	const uint RequestedSamplePerPixel = ENABLE_MULTIPLE_SAMPLES_PER_PIXEL ? SamplesPerPixel : 1;
	uint LocalSamplesPerPixel = RequestedSamplePerPixel;

	if (all(PixelCoord >= LightScissor.xy) && all(PixelCoord <= LightScissor.zw))
	{
		FLightShaderParameters LightParameters = GetRootLightShaderParameters();

		// Read material data
		float3 WorldNormal = 0;
		FOcclusionShadingParameters ShadingParameters = (FOcclusionShadingParameters)0;
		FTransmissionProfileParams TransmissionParameters = (FTransmissionProfileParams)0;
		float SubsurfaceSamplingMeanFreePath = 0;
		bool bIsFirstPerson = false;
		#if SUBSTRATE_GBUFFER_FORMAT==1
		{
			FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
			const FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
			const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));

			const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelCoord);
			
			const uint SSSType = SSSHEADER_TYPE(SSSHeader);
			const bool bHasSSS = SubstratePixelHeader.HasSubsurface();
			const uint BSDFType = SubstratePixelHeader.SubstrateGetBSDFType();

			ShadingParameters.bIsValid = SubstratePixelHeader.IsSubstrateMaterial();
			ShadingParameters.bIsHair = BSDFType == SUBSTRATE_BSDF_TYPE_HAIR;
			ShadingParameters.bIsSubsurface = bHasSSS ? SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP : false;
			ShadingParameters.bHasSubsurfaceProfile = bHasSSS ? (SSSType == SSS_TYPE_DIFFUSION_PROFILE || SSSType == SSS_TYPE_DIFFUSION) : false;
			ShadingParameters.bBackfaceCulling = !(BSDFType == SUBSTRATE_BSDF_TYPE_EYE || (bHasSSS && (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP)));

			// The SSDenoiser reads the top layer normal in order to know how to process the data.
			// RTShadow traces using the top layer normal when NoL>0 only. So if normals for slabs below do not match, light leaking will occur.
			// For instance: a flat translucent top layer above a normal mapped bottom layer.
			// That is why, for materials with multiple local bases, we must not do backface culling.
			if (SubstratePixelHeader.GetLocalBasesCount() > 1)
			{
				// Technically, this could only be done when vertical layering is encountered only. 
				// But we do not have that data available (or it would be costly to infer from closures)
				ShadingParameters.bBackfaceCulling = false;
			}

			WorldNormal = TopLayerData.WorldNormal;
			bIsFirstPerson = SubstratePixelHeader.IsFirstPerson();


			if (ENABLE_TRANSMISSION)
			{
				TransmissionParameters = GetSubstrateSubsurfaceTransmissionProfileParams(SSSType, SSSHeader);

				// If MFP is available, use the actual mean free path, otherwise, fallback to the old behavior using extinction scale in the subsurface profile.
				SubsurfaceSamplingMeanFreePath = TransmissionParameters.ExtinctionScale;

				BRANCH
				if (TransmissionMeanFreePathType == TransmissionMeanFreePathType_MaxMeanFreePath)
				{
					const float RadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
					SubsurfaceSamplingMeanFreePath = SubstrateExtinctionToShadowMFP(TransmissionParameters.ExtinctionScale) * RadiusScale;
				}
			}

			if (ShadingParameters.bIsSubsurface) // Legacy Subsurface shading model == Substrate Wrap SSS.
			{
				FSubsurfaceOpacityMFP SubsurfaceOpacityMFP = SubstrateGetSubsurfaceOpacityMFP(SSSHeader);
				if (SubsurfaceOpacityMFP.bDataIsOpacity)
				{
					TransmissionParameters.ExtinctionScale = SubsurfaceOpacityMFP.Data;
				}
			}
		}	
		#else
		{
			FGBufferData GBufferData = GetGBufferDataFromSceneTexturesLoad(PixelCoord);
			ShadingParameters.bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
			ShadingParameters.bIsHair = GBufferData.ShadingModelID == SHADINGMODELID_HAIR;
			ShadingParameters.bIsSubsurface = GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE;
			ShadingParameters.bHasSubsurfaceProfile = GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE;
			// eye, two sided foliage and subsurface do not need backface culling.
			ShadingParameters.bBackfaceCulling =
				!(GBufferData.ShadingModelID == SHADINGMODELID_EYE ||
				GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE || 
				GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE);

			WorldNormal	= GBufferData.WorldNormal;
			bIsFirstPerson = IsFirstPerson(GBufferData);

			if (ENABLE_TRANSMISSION)
			{
				TransmissionParameters = GetTransmissionProfileParams(ExtractSubsurfaceProfileInt(GBufferData));
				
				// If MFP is available, use the actual mean free path, otherwise, fallback to the old behavior using extinction scale in the subsurface profile.
				SubsurfaceSamplingMeanFreePath = TransmissionParameters.ExtinctionScale;

				BRANCH
				if (TransmissionMeanFreePathType == TransmissionMeanFreePathType_MaxMeanFreePath)
				{
					SubsurfaceSamplingMeanFreePath = DecodeSamplingSSSProfileRadius(GBufferData);
				}
			}

			if (ShadingParameters.bIsSubsurface)
			{
				TransmissionParameters.ExtinctionScale = GBufferData.CustomData.a;
			}
		}
		#endif

		// Mask out depth values that are infinitely far away
		float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
		const bool bIsDepthValid = SceneDepthTexture.Load(int3(PixelCoord, 0)).r > 0.0;
		const bool bIsValidPixel = ShadingParameters.bIsValid && bIsDepthValid;
		const uint LightChannel = GetSceneLightingChannel(PixelCoord);
		const bool bTraceRay = bIsValidPixel && (LightChannel & LightingChannelMask) != 0;
		if (!bTraceRay)
		{
			LocalSamplesPerPixel = 0;
		}

		uint RaytracingMask = RAY_TRACING_MASK_SHADOW | RAY_TRACING_MASK_THIN_SHADOW;
		
		// The first person GBuffer bit is not available when static lighting is enabled, so we skip intersections with the first person world space representation to avoid artifacts in that case.
#if HAS_FIRST_PERSON_GBUFFER_BIT
		// When tracing a ray for a first person pixel, we do not want to intersect the world space representation of that first person primitive.
		// Doing so would lead to "self"-intersection artifacts between the two representations of that mesh.
		if (!bIsFirstPerson)
		{
			RaytracingMask |= RAY_TRACING_MASK_OPAQUE_FP_WORLD_SPACE;
		}
#endif

		Occlusion = ComputeOcclusion(
			PixelCoord,
			RaytracingMask,
			DeviceZ,
			WorldNormal,
			ShadingParameters,
			LightParameters, 
			TransmissionParameters,
			SubsurfaceSamplingMeanFreePath,
			LocalSamplesPerPixel);

		#if USE_HAIR_LIGHTING
		float HairDeviceZ = 0;
		if (NeedTraceHair(PixelCoord, HairDeviceZ))
		{
			FTransmissionProfileParams HairTransmissionParameters = (FTransmissionProfileParams)0;
			FOcclusionShadingParameters HairOcclusionShadingParameters = (FOcclusionShadingParameters)0;
			HairOcclusionShadingParameters.bIsValid = true;
			HairOcclusionShadingParameters.bIsHairStrands = true;

			HairOcclusion = ComputeOcclusion(
				PixelCoord,
				RAY_TRACING_MASK_SHADOW,
				HairDeviceZ,
				float3(1,0,0),
				HairOcclusionShadingParameters,
				LightParameters,
				HairTransmissionParameters,
				0.f/*SubsurfaceSamplingMeanFreePath*/,
				RequestedSamplePerPixel);
		}
		#endif
	}

	// Opaque shadow on ray does not need any denoising due to the high frequency nature of hair.
	#if USE_HAIR_LIGHTING
	{
		// the channel assignment is documented in ShadowRendering.cpp (look for Light Attenuation channel assignment)
		const float HairShadow = OcclusionToShadow(HairOcclusion, RequestedSamplePerPixel);
		const float FadedShadow = HairShadow;
		const float4 OutColor = EncodeLightAttenuation(FadedShadow.xxxx);
		RWSubPixelOcclusionMaskUAV[PixelCoord] = OutColor;
	}
	#endif

	const float Shadow = OcclusionToShadow(Occlusion, LocalSamplesPerPixel);

	if (DIM_DENOISER_OUTPUT == 2)
	{
		RWOcclusionMaskUAV[PixelCoord] = float4(
			Shadow,
			Occlusion.ClosestRayDistance,
			0,
			Occlusion.TransmissionDistance);

	}
	else if (DIM_DENOISER_OUTPUT == 1)
	{
		float AvgHitDistance = -1.0;
		if (Occlusion.HitCount > 0.0)
		{
			AvgHitDistance = Occlusion.SumRayDistance / Occlusion.HitCount;
		}
		else if (Occlusion.RayCount > 0.0)
		{
			AvgHitDistance = 1.0e27;
		}

		// TODO(Denoiser): the denoiser would much prefer a single RG texture.
		RWOcclusionMaskUAV[PixelCoord] = float4(Shadow, Occlusion.TransmissionDistance, Shadow, Occlusion.TransmissionDistance);
		RWRayDistanceUAV[PixelCoord] = AvgHitDistance;
	}
	else
	{
		const float ShadowFadeFraction = 1;
		float SSSTransmission = Occlusion.TransmissionDistance;

		// 0 is shadowed, 1 is unshadowed
		// RETURN_COLOR not needed unless writing to SceneColor;
		float FadedShadow = lerp(1.0f, Shadow, ShadowFadeFraction);
		float FadedSSSShadow = lerp(1.0f, SSSTransmission, ShadowFadeFraction);

		// the channel assignment is documented in ShadowRendering.cpp (look for Light Attenuation channel assignment)
		float4 OutColor;
		if (LIGHT_TYPE == LIGHT_TYPE_DIRECTIONAL)
		{
			OutColor = EncodeLightAttenuation(half4(FadedShadow, FadedSSSShadow, 1.0, FadedSSSShadow));
		}
		else
		{
			OutColor = EncodeLightAttenuation(half4(FadedShadow, FadedSSSShadow, FadedShadow, FadedSSSShadow));
		}

		RWOcclusionMaskUAV[PixelCoord] = OutColor;
	}
}
