// Copyright Epic Games, Inc. All Rights Reserved.


#include "/Engine/Shared/RayTracingTypes.h"
#include "../Common.ush"

#define SUPPORT_CONTACT_SHADOWS		0
#define USE_SOURCE_TEXTURE			1

#define PreIntegratedGF				ReflectionStruct.PreIntegratedGF
#define PreIntegratedGFSampler		GlobalBilinearClampedSampler

#include "../DeferredShadingCommon.ush"
#include "../DeferredLightingCommon.ush"
#include "../ReflectionEnvironmentShared.ush"
#include "../Montecarlo.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush" 
#include "../PathTracing/Material/PathTracingFresnel.ush"
#include "../HeightFogCommon.ush"
#include "../SobolRandom.ush"
#include "../SceneTextureParameters.ush"
#include "RayTracingCommon.ush"
#include "RayTracingDeferredShadingCommon.ush"
#include "RayTracingHitGroupCommon.ush"

#define	ERayTracingPrimaryRaysFlag_None                               0
#define	ERayTracingPrimaryRaysFlag_UseGBufferForMaxDistance    (1u << 0)
#define	ERayTracingPrimaryRaysFlag_PrimaryView                 (1u << 1)
#define	ERayTracingPrimaryRaysFlag_AllowSkipSkySample          (1u << 2)

#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 0
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateEvaluation.ush"

int SamplesPerPixel;
int MaxRefractionRays;
int HeightFog;
int ForceOpaqueRays;
int ShadowsType;
int ShadowsTranslucencyType;
int ShouldDoDirectLighting;
int ShouldDoEmissiveAndIndirectLighting;
int UpscaleFactor;
uint PrimaryRayFlags;

float TranslucencyMinRayDistance;
float TranslucencyMaxRayDistance;
float TranslucencyMaxRoughness;
int  TranslucencyRefraction;
float MaxNormalBias;

Texture2D SceneColorTexture;

RaytracingAccelerationStructure TLAS;
RaytracingAccelerationStructure FarFieldTLAS;

RWTexture2D<float4> ColorOutput;
RWTexture2D<float> RayHitDistanceOutput;

#include "RayTracingLightingCommon.ush"

#define RadianceCacheInterpolation LumenGIVolumeStruct

#include "../Lumen/LumenTranslucencyVolumeShared.ush"

float CalcNoT(float CosTheta1, float N1, float N2)
{
	float SinTheta1_Squared = 1.0 - CosTheta1 * CosTheta1;
	float SinTheta2_Squared = (SinTheta1_Squared * N1 * N1) / (N2 * N2);
	float CosTheta2_Squared = 1.0 - SinTheta2_Squared;
	return CosTheta2_Squared > 0.0 ? sqrt(CosTheta2_Squared) : 0.0;
}

float FresnelDielectric(float Eta, float IoH, float ToH)
{
	float Rs = Square((Eta * IoH - ToH) / (Eta * IoH + ToH));
	float Rp = Square((Eta * ToH - IoH) / (Eta * ToH + IoH));
	return (Rs + Rp) / 2;
}

float3 GetSkyRadiance(float3 Direction, float Roughness)
{
	float SkyAverageBrightness = 1.0f;
	return GetSkyLightReflection(Direction, Roughness, SkyAverageBrightness);
}

RAY_TRACING_ENTRY_RAYGEN(RayTracingPrimaryRaysRGS)
{
	uint2 DispatchThreadId = DispatchRaysIndex().xy + View.ViewRectMin.xy;
	uint2 PixelCoord = GetPixelCoord(DispatchThreadId, UpscaleFactor);
	uint LinearIndex = PixelCoord.y * View.BufferSizeAndInvSize.x + PixelCoord.x; // TODO(Denoiser): PixelCoord or DispatchThreadId
	
	float2 InvBufferSize = View.BufferSizeAndInvSize.zw;
	float2 UV = (float2(PixelCoord) + 0.5) * InvBufferSize;
	
	float GBufferDepth = ConvertFromDeviceZ(SampleDeviceZFromSceneTextures(UV)); // CalcSceneDepth SampleDeviceZFromSceneTextures
	float3 TranslatedWorldPosition = ReconstructTranslatedWorldPositionFromDepth(UV, GBufferDepth);

	// Trace rays from camera origin to Gbuffer (only translucent objects are considered on the first ray, so a bias is not needed)
	FRayDesc Ray = CreatePrimaryRay(UV);
	FRayCone RayCone = (FRayCone)0;
	RayCone.SpreadAngle = View.EyeToPixelSpreadAngle;

	if((ERayTracingPrimaryRaysFlag_UseGBufferForMaxDistance & PrimaryRayFlags) != 0) 
	{
		// NOTE: we need a small epsilon even though we only trace RAY_TRACING_MASK_TRANSLUCENT objects at first
		//       because we might have an object with a mix of materials on it and therefore still need to avoid
		//       hitting the target surface if possible
		Ray.TMax = dot(TranslatedWorldPosition - Ray.Origin, Ray.Direction) - 0.1; // assumes Ray.Direction is normalized
	}

	bool bAllowSkySampling;
	if((ERayTracingPrimaryRaysFlag_AllowSkipSkySample & PrimaryRayFlags) != 0) 
	{
		// Sky is only sampled when infinite reflection rays are used.
		bAllowSkySampling =  TranslucencyMaxRayDistance < 0;
	} 
	else 
	{
		bAllowSkySampling = true;
	}
	// Check if the Sky Light should affect reflection rays within translucency.
	const bool bSkyLightAffectReflection = ShouldSkyLightAffectReflection();
	
	const bool bPrimaryView = (ERayTracingPrimaryRaysFlag_PrimaryView & PrimaryRayFlags) != 0;
	bool bHasScattered = false;
	float BackgroundVisibility = 0.0;

	// Integrated data by path tracing
	float3 PathRadiance = 0.0;
	float3 PathThroughput = 1.0;
	float LastRoughness = 0.0;
	float HitDistance = 0.0f;

	FRandomSequence RandSequence = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), 0, 1);
	for (uint RefractionRayIndex = 0; RefractionRayIndex < MaxRefractionRays; ++RefractionRayIndex)
	{
		const bool bIsCameraRay = !bHasScattered;
		const uint RefractionRayFlags = (bIsCameraRay ? RAY_FLAG_CULL_BACK_FACING_TRIANGLES : 0) | (ForceOpaqueRays ? RAY_FLAG_FORCE_OPAQUE : 0);
		const uint RefractionInstanceInclusionMask = (!bPrimaryView && bIsCameraRay) || !TranslucencyRefraction ? RAY_TRACING_MASK_TRANSLUCENT : (RAY_TRACING_MASK_OPAQUE | RAY_TRACING_MASK_TRANSLUCENT | RAY_TRACING_MASK_HAIR_STRANDS);
		const bool bRefractionRayTraceSkyLightContribution = false;
		const bool bRefractionDecoupleSampleGeneration = true;
#if PROJECT_SUPPORTS_LUMEN
		const bool bRefractionEnableSkyLightContribution = !IsLumenTranslucencyGIEnabled();
#else
		const bool bRefractionEnableSkyLightContribution = true;
#endif
		float3 PathVertexRadiance = float3(0, 0, 0);
		float MaxTraceDistance = 100000.0f;


		FMaterialClosestHitPayload Payload = TraceRayAndAccumulateResults(
			Ray,
			TLAS,
			FarFieldTLAS,
			RefractionRayFlags,
			RefractionInstanceInclusionMask,
			RandSequence,
			PixelCoord,
			MaxNormalBias,
			MaxTraceDistance,
			ShadowsType,
			ShadowsTranslucencyType,
			ShouldDoDirectLighting,
			ShouldDoEmissiveAndIndirectLighting,
			bRefractionRayTraceSkyLightContribution,
			bRefractionDecoupleSampleGeneration,
			bIsCameraRay,
			RayCone,
			bRefractionEnableSkyLightContribution,
			PathVertexRadiance);

		float PayloadRoughness = 0;
		float3 PayloadDiffuseColor = 0;
		float3 PayloadSpecularColor = 0;
		float3 PayloadWorldTangent = 0;
		float PayloadAnisotropy = 0;
		float3 PayloadLuminanceWeight = 1.0;
	#if SUBSTRATE_GBUFFER_FORMAT==1
		
		float3 SubstrateMaterialThroughput = 1.0;
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(0, 0, 0);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Payload.SubstrateData, SubstrateAddressing, Payload.SubstrateData);

		BRANCH
		if (SubstratePixelHeader.ClosureCount > 0) // With ray tracing, we only account for the first BSDF since all materials should be simplified to a single Closure.
		{
			const bool bSkipSSSMaterialOverride = true;
			const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Payload.SubstrateData, SubstrateAddressing, SubstratePixelHeader, bSkipSSSMaterialOverride);
			
			if (SubstrateIsBSDFVisible(BSDF))
			{
				// Create the BSDF context
				FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, -Ray.Direction);

				PayloadRoughness = SLAB_ROUGHNESS(BSDF);
				PayloadDiffuseColor = SubstrateGetBSDFDiffuseColor(BSDF);
				PayloadSpecularColor = SubstrateGetBSDFSpecularF0(BSDF);
				PayloadWorldTangent = SubstrateBSDFContext.X;
				PayloadAnisotropy = SLAB_ANISOTROPY(BSDF);

				PayloadLuminanceWeight = BSDF.LuminanceWeightV;
				const float Coverage = saturate(Payload.Opacity);
				SubstrateMaterialThroughput = (1.0 - Coverage) + Coverage * Payload.TransmittancePreCoverage;
			}

		}

	#else

		PayloadRoughness = Payload.Roughness;
		PayloadDiffuseColor = Payload.DiffuseColor;
		PayloadSpecularColor = Payload.SpecularColor;
		PayloadWorldTangent = Payload.WorldTangent;
		PayloadAnisotropy = Payload.Anisotropy;
		PayloadLuminanceWeight = Payload.Opacity;

	#endif

		LastRoughness = PayloadRoughness;

		//
		// Handle no hit condition
		//
		if (Payload.IsMiss())
		{
			if (bHasScattered && bAllowSkySampling)
			{
				// We only sample the sky if the ray has scattered (i.e. been refracted or reflected). Otherwise we are going ot use the regular scene color.
				PathRadiance += PathThroughput * GetSkyRadiance(Ray.Direction, LastRoughness);
			}
			break;
		}
		float3 TranslatedHitPoint = Ray.Origin + Ray.Direction * Payload.HitT;
		float NextMaxRayDistance = Ray.TMax - Payload.HitT;

		//
		// Handle surface lighting
		//
		
		const float3 vertexRadianceWeight = PayloadLuminanceWeight;	// Opacity as coverage. This works for RAY_TRACING_BLEND_MODE_OPAQUE and RAY_TRACING_BLEND_MODE_TRANSLUCENT.

		// It is also needed for RAY_TRACING_BLEND_MODE_ADDITIVE and  RAY_TRACING_BLEND_MODE_ALPHA_COMPOSITE: radiance contribution is alway weighted by coverage.
		// #dxr_todo: I have not been able to setup a material using RAY_TRACING_BLEND_MODE_MODULATE.

		//
		// Apply fog on primary ray (the first translucent surface encountered) if needed.
		// We can only handle primary rays because height and volumetric fog are only setup for the camera point of view.
		// 
		float4 ViewFogInScatteringAndTransmittance = float4(0.0, 0.0, 0.0, 1.0);
		if (HeightFog > 0)
		{
			float3 WorldPositionRelativeToCameraPrimaryRay = PrimaryView.TranslatedWorldCameraOrigin - TranslatedHitPoint;

			// We always sample the height fog to make sure all the intersected transparent surface get affected by it.
			// We cannot only do it for the primary ray otherwise refelction on secondary surface will look too bright. 
			// For that sample the fog for the hit position and view assuming no refraction. This seems acceptable.
			ViewFogInScatteringAndTransmittance = CalculateHeightFog(WorldPositionRelativeToCameraPrimaryRay);

			if (FogStruct.ApplyVolumetricFog > 0
				&& !bHasScattered) // We can only sample the volumetric fog contribution (always camera aligned) if the ray is not scattered around
			{
				float4 ClipPos = mul(float4(TranslatedHitPoint, 1.0f), PrimaryView.TranslatedWorldToClip);
				float3 VolumeUV = ComputeVolumeUVFromNDC(ClipPos);

				ViewFogInScatteringAndTransmittance = CombineVolumetricFog(ViewFogInScatteringAndTransmittance, VolumeUV, 0 /*EyeIndex*/, GBufferDepth);
			}
		}

		PathRadiance += PathThroughput * vertexRadianceWeight * (PathVertexRadiance * ViewFogInScatteringAndTransmittance.a + ViewFogInScatteringAndTransmittance.rgb);

		const bool bIsHoldout = Payload.IsHoldout();
		if (bIsHoldout)
		{
			// A holdout object acts as if it was a background pixel so that it influences the alpha channel
			BackgroundVisibility += Luminance(PathThroughput * vertexRadianceWeight);
		}

		//
		// Handle reflection tracing with a ray per vertex of the refraction path
		//

		// Shorten the rays on rougher surfaces between user-provided min and max ray lengths.
		// When a shortened ray misses the geometry, we fall back to local reflection capture sampling (similar to SSR).
		const float LocalMaxRayDistance = bAllowSkySampling ? 1e27f : lerp(TranslucencyMaxRayDistance, TranslucencyMinRayDistance, PayloadRoughness);
		if (PayloadRoughness < TranslucencyMaxRoughness)
		{
			// Trace reflection ray 
			float2 RandSample = RandSequence.Get2D();

			FRayDesc ReflectionRay;
			ReflectionRay.TMin = 0.01;
			ReflectionRay.TMax = LocalMaxRayDistance;
			ReflectionRay.Origin = TranslatedHitPoint;

			ModifyGGXAnisotropicNormalRoughness(PayloadWorldTangent, PayloadAnisotropy, PayloadRoughness, Payload.WorldNormal, Ray.Direction);

			ReflectionRay.Direction = GenerateReflectedRayDirection(Ray.Direction, Payload.WorldNormal, PayloadRoughness, RandSample);
			ApplyPositionBias(ReflectionRay, Payload.WorldNormal, MaxNormalBias);

			const uint ReflectionRayFlags = RAY_FLAG_CULL_BACK_FACING_TRIANGLES | (ForceOpaqueRays ? RAY_FLAG_FORCE_OPAQUE : 0);
			const uint ReflectionInstanceInclusionMask = (RAY_TRACING_MASK_OPAQUE_REFLECTION | RAY_TRACING_MASK_TRANSLUCENT | RAY_TRACING_MASK_HAIR_STRANDS);
			const bool bReflectionRayTraceSkyLightContribution = false;
			const bool bReflectionDecoupleSampleGeneration = true;
			const bool bReflectionEnableSkyLightContribution = bSkyLightAffectReflection;
			float3 ReflectionRadiance = float3(0, 0, 0);

			FMaterialClosestHitPayload ReflectionPayload = TraceRayAndAccumulateResults(
				ReflectionRay,
				TLAS,
				FarFieldTLAS,
				ReflectionRayFlags,
				ReflectionInstanceInclusionMask,
				RandSequence,
				PixelCoord,
				MaxNormalBias,
				MaxTraceDistance,
				ShadowsType,
				ShadowsTranslucencyType,
				ShouldDoDirectLighting,
				ShouldDoEmissiveAndIndirectLighting,
				bReflectionRayTraceSkyLightContribution,
				bReflectionDecoupleSampleGeneration,
				false,
				RayCone,
				bReflectionEnableSkyLightContribution,
				ReflectionRadiance);

			// If we have not hit anything, sample the distance sky radiance.
			if (ReflectionPayload.IsMiss())
			{
				ReflectionRadiance = GetSkyRadiance(ReflectionRay.Direction, LastRoughness);
			}

			// #dxr_todo: reflection IOR and clear coat also? This only handles default material.
			float NoV = saturate(dot(-Ray.Direction, Payload.WorldNormal));
			const float3 ReflectionThroughput = EnvBRDF(PayloadSpecularColor, PayloadRoughness, NoV);

			// For reflections, we can only sample the height fog (volumetric fog is only for primary rays)
			if (HeightFog > 0)
			{
				float3 OriginToCollider = ReflectionPayload.TranslatedWorldPos - ReflectionRay.Origin;
				float4 ReflectionFogInscatteringAndTransmittance = CalculateHeightFog(OriginToCollider);
				ReflectionRadiance = ReflectionRadiance * ReflectionFogInscatteringAndTransmittance.a + ReflectionFogInscatteringAndTransmittance.rgb;
			}

			PathRadiance += PathThroughput * vertexRadianceWeight * ViewFogInScatteringAndTransmittance.a * ReflectionThroughput * ReflectionRadiance;
		}

		{
			float3 VolumeLighting = 0;

			const float3 TranslucencyEvaluationTranslatedPosition = TranslatedHitPoint;

#if PROJECT_SUPPORTS_LUMEN
			if (IsLumenTranslucencyGIEnabled() && ShouldDoEmissiveAndIndirectLighting)
			{
				// Lumen Dynamic GI + shadowed Skylight
				FDFVector3 TranslucencyEvaluationPosition = DFFastSubtract(TranslucencyEvaluationTranslatedPosition, PrimaryView.PreViewTranslation);

				FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(TranslucencyEvaluationPosition, PrimaryView.WorldToClip, true);

				// Diffuse convolution
				FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(Payload.WorldNormal, 1);
				VolumeLighting.rgb += max(half3(0, 0, 0), DotSH(TranslucencyGISH, DiffuseTransferSH)) / PI;

				const bool bEvaluateBackface = GetShadingModelRequiresBackfaceLighting(Payload.ShadingModelID);
				if (bEvaluateBackface)
				{
					FTwoBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH(-Payload.WorldNormal, 1);
					VolumeLighting.rgb += max(half3(0, 0, 0), DotSH(TranslucencyGISH, SubsurfaceTransferSH)) / PI;
				}
			}
			else
#endif
			{
				// noop => if lumen is not enabled: bRefractionEnableSkyLightContribution == true
			}

			PathRadiance += PathThroughput * vertexRadianceWeight * VolumeLighting * PayloadDiffuseColor;
		}
		
		float3 RefractedDirection = Ray.Direction;
	#if SUBSTRATE_GBUFFER_FORMAT==1

		bHasScattered = false;
		PathThroughput *= SubstrateMaterialThroughput;

		if (bIsHoldout)
		{
			// In the refraction case, the non-opaque part should also contribute to the holdout
			// The net result is that the entire object becomes opaque for the alpha channel.
			BackgroundVisibility += Luminance(PathThroughput) * (1.0 - Payload.Opacity);
			PathThroughput = 0.0;
			break;
		}

	#else // SUBSTRATE_GBUFFER_FORMAT==1

		// Update the refraction/transparency path transmittance and check stop condition
		switch (Payload.BlendingMode)
		{
			case RAY_TRACING_BLEND_MODE_ADDITIVE:
			{
				break;
			}
			case RAY_TRACING_BLEND_MODE_MODULATE:
			{
				PathThroughput *= Payload.Radiance;
				break;
			}
			case RAY_TRACING_BLEND_MODE_ALPHA_COMPOSITE:
			case RAY_TRACING_BLEND_MODE_TRANSLUCENT:
			case RAY_TRACING_BLEND_MODE_ALPHA_HOLDOUT:
			{
				PathThroughput *= 1.0 - Payload.Opacity;
				break;
			}
			default:
			{
				PathThroughput *= 0.0;
				break;
			}
		}
		
		if (all(PathThroughput <= 0.0))
		{
			break;
		}

		//
		// Handle refraction through the surface.
		//

		// Set refraction ray for next iteration
		if (Payload.ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT)
		{
			const float3 N = Payload.WorldNormal;
			const float3 V = -Ray.Direction;
			const float NoV = dot(N, V);

			const float F0 = F0RGBToF0(Payload.SpecularColor);
			const float F = FresnelReflectance(NoV, 1.0, F0);
			const float3 Transmittance = Payload.CustomData.xyz;
			const float PathLength = rcp(saturate(abs(NoV) + 1e-5));
			PathThroughput *= pow(Transmittance, PathLength) * Pow2(1 - F);
			bHasScattered = false; // need to consider both refraction and translucency through the material by coverage.
			if (bIsHoldout)
			{
				// In the refraction case, the non-opaque part should also contribute to the holdout
				// The net result is that the entire object becomes opaque for the alpha channel.
				BackgroundVisibility += Luminance(PathThroughput) * (1.0 - Payload.Opacity);
				PathThroughput = 0.0;
				break;
			}
		}
		else if (TranslucencyRefraction && Payload.BlendingMode == RAY_TRACING_BLEND_MODE_TRANSLUCENT && Payload.Ior > 0.0)
		{
			if (bIsHoldout)
			{
				// In the refraction case, the non-opaque part should also contribute to the holdout
				// The net result is that the entire object becomes opaque for the alpha channel.
				BackgroundVisibility += Luminance(PathThroughput) * (1.0 - Payload.Opacity);
				PathThroughput = 0.0;
				break;
			}
			float Ior = Payload.Ior;
			bHasScattered |= Ior > 1.0 ? true : false;

			bool bIsEntering = Payload.IsFrontFace();

			float Eta = bIsEntering ? Ior : rcp(Ior);

			float3 N = Payload.WorldNormal;
			float3 V = -Ray.Direction;
			float NoV = dot(N, V);

			// Hack to allow one-sided materials to be modeled as dielectrics
			if (NoV < 0.0)
			{
				NoV = -NoV;
				N = -N;
				bIsEntering = true;
			}

			float F0 = F0RGBToF0(Payload.SpecularColor);


			float F = 0.0; // fresnel
			SampleRefraction(-V, N, Eta, 1.0 /* always sample refraction or TIR */, RefractedDirection, F);
			PathThroughput *= F;

			// ray has bent, so it may need to go arbitrarily far
			NextMaxRayDistance = LocalMaxRayDistance;
		}

	#endif // SUBSTRATE_GBUFFER_FORMAT==1


		//
		// Setup refracted ray to be traced
		//

		Ray.Origin = TranslatedHitPoint;
		Ray.TMin = 0.01;
		Ray.TMax = NextMaxRayDistance;
		Ray.Direction = RefractedDirection;
		float SurfaceCurvature = 0.0f; /* #todo_dxr assume no curvature */
		RayCone = PropagateRayCone(RayCone, SurfaceCurvature, GBufferDepth);
	}

	if (!bHasScattered)
	{
		// If the ray did not bend, composite over the SceneColor texture and accumulate alpha
		if (bPrimaryView)
		{
			// our path reached the background, accumulate its contribution for the alpha channel
			BackgroundVisibility += Luminance(PathThroughput);
		}
		else
		{
			float4 SceneColorContrib = SceneColorTexture.SampleLevel(GlobalPointClampedSampler, UV, 0);
			PathRadiance += PathThroughput * SceneColorContrib.xyz / View.PreExposure;
			BackgroundVisibility += Luminance(PathThroughput) * SceneColorContrib.w;
		}
	}

	// NOTE: UE by convention tracks background visibility which is (1.0-AccumulatedOpacity) (the complement of traditional alpha)
	const float FinalAlpha = saturate(BackgroundVisibility);

	PathRadiance *= View.PreExposure;
	PathRadiance = ClampToHalfFloatRange(PathRadiance);
	ColorOutput[DispatchThreadId] = float4(PathRadiance, FinalAlpha);
	RayHitDistanceOutput[DispatchThreadId] = HitDistance;
	
}
