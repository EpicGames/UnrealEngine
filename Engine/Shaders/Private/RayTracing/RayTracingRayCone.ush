// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FRayCone
{
	float Width;
	float SpreadAngle;
};

FRayCone UnpackRayCone(uint PackedRayCone)
{
	FRayCone Result;
	Result.Width       = f16tof32(PackedRayCone & 0xFFFF);
	Result.SpreadAngle = f16tof32(PackedRayCone >> 16);
	return Result;
}

uint PackRayCone(FRayCone RayCone)
{
	return f32tof16(RayCone.Width) | (f32tof16(RayCone.SpreadAngle) << 16);
}

FRayCone PropagateRayCone(in FRayCone Cone, in float SurfaceSpreadAngle, in float  HitT)
{
	FRayCone NewCone;
	NewCone.Width = Cone.SpreadAngle * HitT + Cone.Width;
	NewCone.SpreadAngle = Cone.SpreadAngle + SurfaceSpreadAngle;
	return NewCone;
}