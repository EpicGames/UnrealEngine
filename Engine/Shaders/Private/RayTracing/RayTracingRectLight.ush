// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../LightShaderParameters.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "../RectLight.ush"

bool GenerateRectLightOcclusionRay(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPdf
)
{
	RayOrigin = TranslatedWorldPosition;
	RayDirection = float3(0,0,0);
	RayTMin = 0.0;
	RayTMax = 0.0;
	RayPdf = 0.0;

	// Define rectangle
	FRect Rect = GetRect(LightParameters, TranslatedWorldPosition);

	// 1. Derive the pdf when uniformly sampling from the spherical rectangle
	FSphericalRect SphericalRect = BuildSphericalRect(Rect);
	RayPdf = 1.0 / SphericalRect.SolidAngle;

	// 2. Uniformly sample from the spherical rectangle
	FSphericalRectSample Sample = UniformSampleSphericalRect(RandSample, SphericalRect);

	// Light-normal culling
	bool bIsValidRay = dot(Sample.Direction, LightParameters.Direction) >= 0.0;
	
	// Apply normal perturbation when defining ray. normalize() resolves several banding artifacts
	// at the cost of creating some bias under idea lighting condition but is neglectable.
	RayDirection = Sample.Direction;
	RayTMax = Sample.Distance;

	return bIsValidRay;
}