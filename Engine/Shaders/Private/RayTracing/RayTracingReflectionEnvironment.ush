// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Common header to access reflection capture data in ray tracing shaders.
* Requires:
*  - ReflectionCaptureUniformBuffer
*  - ForwardLightDataUniformBuffer
*/

// Setup the reflection composite to sample the sky and reflection captures.
#define REFLECTION_COMPOSITE_HAS_BOX_CAPTURES 1
#define REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES 1
#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES 1
#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 1
// Enable skly light sampling as last fallback after reflection captures
#ifdef ENABLE_SKY_LIGHT
#undef ENABLE_SKY_LIGHT
#endif
#define ENABLE_SKY_LIGHT 1
// Disable static lighting as this is not needed in this case.
#ifdef ALLOW_STATIC_LIGHTING
#undef ALLOW_STATIC_LIGHTING
#endif
#define ALLOW_STATIC_LIGHTING 0
// Also specify that we are going to go over a range of reflection captures without culling available.
// All reflection captures available are always available here even if completely outside the view.
#define REFLECTION_COMPOSITE_NO_CULLING_DATA
#include "../ReflectionEnvironmentShared.ush"
#include "../ReflectionEnvironmentComposite.ush"

