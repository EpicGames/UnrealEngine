// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file contains utility methods that can be used by ray tracing shaders (such as the CHS/AHS pairs, light miss shaders, etc ...)

#ifndef RAYHITGROUPSHADER
#error "This header should only be included in raytracing contexts"
#endif

#ifndef PATH_TRACING // Path Tracing has a similar implemental with a slightly different set of flags

#include "RayTracingCommon.ush"

static int CurrentPayloadInputFlags = 0;

bool GetShadowReplaceState()
{
	return (CurrentPayloadInputFlags & RAY_TRACING_PAYLOAD_INPUT_FLAG_SHADOW_RAY) != 0;
}

float IsShadowDepthShader()
{
	return GetShadowReplaceState() ? 1.0f : 0.0f;
}

#endif