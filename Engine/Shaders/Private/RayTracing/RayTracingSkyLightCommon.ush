// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RayTracingSkyLightCommon.ush: Common functions for MipTree-based SkyLight visibility ray sampling
===============================================================================================*/

#pragma once

#include "../MonteCarlo.ush"

// shader parameters for skylight integration
Texture2D SkylightTexture;
Texture2D<float> SkylightPdf;
SamplerState SkylightTextureSampler;
float SkylightInvResolution;
int SkylightMipCount;

#define USE_HIERARCHICAL_IMPORTANCE_SAMPLING		1

float SkyLight_Estimate()
{
	// cancels out constant factor in pdf below
#if USE_HIERARCHICAL_IMPORTANCE_SAMPLING
	return 4 * PI * SkylightPdf.Load(int3(0, 0, SkylightMipCount - 1)); 
#else
	return 4 * PI;
#endif
}

// Returns: Radiance and Pdf for the specified direction
float4 SkyLight_EvalLight(float3 Dir)
{
	// NOTE: assumes direction is normalized
	float2 UV = InverseEquiAreaSphericalMapping(Dir.yzx);
	float4 Result = SkylightTexture.SampleLevel(SkylightTextureSampler, UV, 0);
	float3 Radiance = Result.xyz;
#if USE_HIERARCHICAL_IMPORTANCE_SAMPLING
	float Pdf = Result.w > 0 ? Result.w / (4 * PI * SkylightPdf.Load(int3(0, 0, SkylightMipCount - 1))) : 0.0; 
#else
	float Pdf = 1.0 / (4.0 * PI);
#endif
	return float4(Radiance, Pdf);
}

struct FSkyLightSample {
	float3 Direction;
	float3 Radiance;
	float Pdf;
};

FSkyLightSample SkyLight_SampleLight(float2 RandSample)
{
#if USE_HIERARCHICAL_IMPORTANCE_SAMPLING
	float2 UV = RandSample;

	int3 Pixel = int3(0, 0, SkylightMipCount - 2);
	for (; Pixel.z >= 0; Pixel.z--)
	{
		Pixel.xy *= 2;
		// TODO: would be nice to have GatherRed available to do this in a single lookup ...
		float P00 = SkylightPdf.Load(Pixel + int3(0, 0, 0));
		float P10 = SkylightPdf.Load(Pixel + int3(1, 0, 0));
		float P01 = SkylightPdf.Load(Pixel + int3(0, 1, 0));
		float P11 = SkylightPdf.Load(Pixel + int3(1, 1, 0));

		float L = P00 + P01;
		float R = P10 + P11;

		float ProbX = L / (L + R);
		if (UV.x < ProbX)
		{
			UV.x /= ProbX;
			float ProbY = P00 / L;
			if (UV.y < ProbY)
			{
				UV.y /= ProbY;
			}
			else
			{
				Pixel.y++;
				UV.y = (UV.y - ProbY) / (1 - ProbY);
			}
		}
		else
		{
			Pixel.x++;
			UV.x = (UV.x - ProbX) / (1 - ProbX);
			float ProbY = P10 / R;
			if (UV.y < ProbY)
			{
				UV.y /= ProbY;
			}
			else
			{
				Pixel.y++;
				UV.y = (UV.y - ProbY) / (1 - ProbY);
			}
		}
	}

	Pixel.z = 0;
	float4 Result = SkylightTexture.Load(Pixel);
	float3 Radiance = Result.xyz;
	float OutPdf = Result.w / (4 * PI * SkylightPdf.Load(int3(0, 0, SkylightMipCount - 1)));

	UV = (float2(Pixel.xy) + UV) * SkylightInvResolution;

	FSkyLightSample Sample;
	Sample.Direction = EquiAreaSphericalMapping(UV).zxy;
	Sample.Radiance = Radiance;
	Sample.Pdf = OutPdf;
	return Sample;
#else
	FSkyLightSample Sample;

	Sample.Direction = UniformSampleSphere(RandSample).xyz;
	Sample.Pdf = 1.0 / (4.0 * PI);
	float2 UV = InverseEquiAreaSphericalMapping(Sample.Direction.yzx);
	Sample.Radiance = SkylightTexture.SampleLevel(SkylightTextureSampler, UV, 0).xyz;
	return Sample;
#endif
}
