// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	RayTracingSkyLightEvaluation.ush: Common functions for SkyLight ray traced evaluation
===============================================================================================*/

#pragma once

#include "../Common.ush"
#include "../ShadingModels.ush"
#include "../DeferredShadingCommon.ush"
#include "../ReflectionEnvironmentShared.ush"
#include "RayTracingCommon.ush"
#include "RayTracingSkyLightCommon.ush"
#include "SkyLightVisibilityRaysData.ush"

uint3 SkyLightVisibilityRaysDimensions;
StructuredBuffer<FSkyLightVisibilityRays> SkyLightVisibilityRays;

#ifndef USE_HAIR_LIGHTING
#define USE_HAIR_LIGHTING 0
#endif

#if USE_HAIR_LIGHTING
#include "../HairStrands/HairStrandsRaytracing.ush"
#endif

// Evaluates the current sky light at a given location on a surface by integrating over the sky light via casting occlusion rays
// SampleCoord - Used for accessing the decoupled visibility ray buffer and should reflect some sort of uniformly distributed 2d coordinate to properly importance sample from the sky light. If bGBufferSampleOrigin is true this must be
//               the pixel location of the g-buffer sample for proper camera ray bias and depth reconstruction.
// TranslatedWorldPosition - Unbiased position on a surface to sample from.
// WorldNormal - Normal of the surface to sample from.
// ViewDirection - Normalized vector from the viewing source in the direction of observation of the point to sample from
// GBufferData - GBuffer data to use for surface BxDF evaluation.
// bGBufferSampleOrigin - Indicates that the sample coordinate and world position were acquired from a g-buffer sample and that camera relative depth biasing should be used instead of just normal biased.
// DeviceZ - The DeviceZ value from the g-buffer's depth buffer if bGBufferSampleOrigin is set to true, otherwise ignored.
// bDecoupleSampleGeneration - Indicates weather or not to use decoupled sample generation for generating visibility rays, requires that the visibility ray buffer has been populated by the GenerateSkyLightVisibilityRaysCS pass.

// UE_DEPRECATED 5.7 - Deprecated by Substrate
void SkyLightEvaluate(
	in const uint2 SampleCoord,
	in const uint2 PixelCoord,
	in const uint SamplesPerPixel,
	in const float3 TranslatedWorldPosition,
	in const float3 WorldNormal,
	in const float3 ViewDirection,
	in const FGBufferData GBufferData,
	in const RaytracingAccelerationStructure TLAS,
	in const bool bGBufferSampleOrigin,
	in const float DeviceZ,
	in const bool bDecoupleSampleGeneration,
	inout float3 ExitantRadiance,
	inout float3 DiffuseExitantRadiance,
	inout float AmbientOcclusion,
	inout float HitDistance)
{
	float3 CurrentWorldNormal = WorldNormal;
	float RayDistance = 0.0;
	float HitCount = 0.0;
	float3 BentNormal = float3(0.0f, 0.0f, 0.0f);

	// Initialize the output exitant radiance, diffuse exitant radiance, ambient occlusion and hit distance
	ExitantRadiance = float3(0.0f, 0.0f, 0.0f);
	DiffuseExitantRadiance = float3(0.0f, 0.0f, 0.0f);
	AmbientOcclusion = 0.0f;
	HitDistance = -1.0f;

	// split samples between strategies unless the skylight pdf is 0 due to MIS compensation (which implies a constant map)
	const float SkyLightSamplingStrategyPdf = SkyLight_Estimate() > 0 ? 0.5 : 0.0;

	// Iterate up to the requested sample count
	for (uint SampleIndex = 0; SampleIndex < SamplesPerPixel; ++SampleIndex)
	{
		FRayDesc Ray;
		float RayWeight;

		if (bDecoupleSampleGeneration)
		{
			// Get a visibility ray for the current sample from the precomputed visibility ray buffer

			const uint SkyLightVisibilityRayIndex = GetSkyLightVisibilityRayTiledIndex(SampleCoord, SampleIndex, SkyLightVisibilityRaysDimensions.xy);
			FSkyLightVisibilityRays SkyLightVisibilityRay = SkyLightVisibilityRays[SkyLightVisibilityRayIndex];

			Ray.Origin = TranslatedWorldPosition;
			Ray.Direction = SkyLightVisibilityRay.DirectionAndPdf.xyz;
			Ray.TMin = 0.0;
			Ray.TMax = SkyLight.MaxRayDistance;
			RayWeight = SkyLightVisibilityRay.DirectionAndPdf.w;
		}
		else
		{
			FRandomSequence RandSequence = RandomSequenceCreate(uint3(PixelCoord, View.StateFrameIndex), SampleIndex, SamplesPerPixel);

			// Determine sky light or lambert ray
			float2 RandSample = RandSequence.Get2D();

			// Generate a visibility ray for the current sample
			float SkyLightPdf = 0;
			float CosinePdf = 0;
			BRANCH
			if (RandSample.x < SkyLightSamplingStrategyPdf)
			{
				RandSample.x /= SkyLightSamplingStrategyPdf;

				FSkyLightSample SkySample = SkyLight_SampleLight(RandSample);
				Ray.Direction = SkySample.Direction;
				SkyLightPdf = SkySample.Pdf;

				CosinePdf = saturate(dot(CurrentWorldNormal, Ray.Direction)) / PI;
			}
			else
			{
				RandSample.x = (RandSample.x - SkyLightSamplingStrategyPdf) / (1.0 - SkyLightSamplingStrategyPdf);

				float4 CosSample = CosineSampleHemisphere(RandSample, CurrentWorldNormal);
				Ray.Direction = CosSample.xyz;
				CosinePdf = CosSample.w;

				SkyLightPdf = SkyLight_EvalLight(Ray.Direction).w;
			}

			Ray.Origin = TranslatedWorldPosition;
			Ray.TMin = 0.0;
			Ray.TMax = SkyLight.MaxRayDistance;
			// MIS / pdf
			RayWeight = 1.0 / lerp(CosinePdf, SkyLightPdf, SkyLightSamplingStrategyPdf);
		}

		// Recompute the current WorldNormal if lighting a hair fiber
		if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
		{
			const float3 LightDirection = Ray.Direction;
			const float3 TangentDirection = normalize(CurrentWorldNormal);

			CurrentWorldNormal = normalize(LightDirection - TangentDirection * dot(LightDirection, TangentDirection));
		}

		// Apply a depth bias based on if the sample world position came from the g-buffer or not
		float NoL = dot(CurrentWorldNormal, Ray.Direction);
		if (NoL > 0.0)
		{
			if (bGBufferSampleOrigin)
			{
				ApplyCameraRelativeDepthBias(Ray, PixelCoord, DeviceZ, CurrentWorldNormal, SkyLight.MaxNormalBias);
			}
			else
			{
				ApplyPositionBias(Ray, CurrentWorldNormal, SkyLight.MaxNormalBias);
			}
		}
		else
		{
			ApplyPositionBias(Ray, -CurrentWorldNormal, SkyLight.MaxNormalBias);
		}
		NoL = saturate(NoL);

		// Trace a visibility ray
		uint RayFlags = 0;
		const uint InstanceInclusionMask = RAY_TRACING_MASK_SHADOW;

#if !ENABLE_MATERIALS
		RayFlags |= RAY_FLAG_FORCE_OPAQUE;
#endif
#if !ENABLE_TWO_SIDED_GEOMETRY
		RayFlags |= RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
#endif

		FMinimalPayload MinimalPayload = TraceVisibilityRay(
			TLAS,
			RayFlags,
			InstanceInclusionMask,
			Ray);

		#if USE_HAIR_LIGHTING
		if (GBufferData.ShadingModelID != SHADINGMODELID_HAIR)
		{
			const float HairOcclusionThreshold = 1;
			uint LinearIndex = CalcLinearIndex(PixelCoord);
			FRandomSequence RandSequence = RandomSequenceCreate(LinearIndex, View.StateFrameIndex); // TODO: should include the SampleIndex here
			MinimalPayload.HitT = TraverseHair(PixelCoord, RandSequence, Ray.Origin, Ray.Direction, MinimalPayload.HitT, VirtualVoxel.Raytracing_SkyOcclusionThreshold);
		}
		#endif

		if (MinimalPayload.IsHit())
		{
			RayDistance += MinimalPayload.HitT;
			HitCount += 1.0;
		}
		else
		{
			BentNormal += Ray.Direction;

			// Evaluate material
			const half3 N = WorldNormal;
			const half3 V = -ViewDirection;
			const half3 L = Ray.Direction;
			FDirectLighting LightingSample;
			if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
			{
				float Shadow = 0.0;
				float Backlit = 0.0;
				float Area = 0.0;
				uint2 Random = 0;
				FHairTransmittanceData HairTransmittance = InitHairTransmittanceData(false);
				LightingSample.Diffuse = HairShading(GBufferData, L, V, N, Shadow, HairTransmittance, Backlit, Area, Random);
				LightingSample.Transmission = 0;
				LightingSample.Specular = 0;
			}
			else
			{
				FShadowTerms ShadowTerms = { 0.0, 0.0, 0.0, InitHairTransmittanceData() };
				LightingSample = EvaluateBxDF(GBufferData, N, V, L, NoL, ShadowTerms);
			}
			float3 Brdf = LightingSample.Diffuse + LightingSample.Transmission + LightingSample.Specular;
			float3 IncomingRadiance = SkyLight_EvalLight(Ray.Direction).xyz;

			ExitantRadiance += IncomingRadiance * Brdf * RayWeight;
			float3 DiffuseThroughput = LightingSample.Diffuse;
			if (SkyLight.bTransmission)
			{
				DiffuseThroughput += LightingSample.Transmission;
			}
			DiffuseExitantRadiance += IncomingRadiance * DiffuseThroughput * RayWeight;
		}
	}

	// Average values over the number of samples
	if (SamplesPerPixel > 0)
	{
		const float SamplesPerPixelInv = rcp(SamplesPerPixel);
		ExitantRadiance *= SamplesPerPixelInv;
		DiffuseExitantRadiance *= SamplesPerPixelInv;
		AmbientOcclusion = HitCount * SamplesPerPixelInv;
	}

	// Normalize the bent normal if it is not the zero vector
	if (all(BentNormal == 0.0f))
	{
		BentNormal = WorldNormal;
	}
	else
	{
		BentNormal = normalize(BentNormal);
	}

	// Calculate the hit distance if any occluding geometry was hit
	if (HitCount > 0.0)
	{
		HitDistance = RayDistance / HitCount;
	}

	// Add multi-scattering contribution
	if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR) {
		const half3 N = WorldNormal;
		const half3 V = -ViewDirection;
		const half3 L = BentNormal;

		// Trace a visibility ray from light to shading point to get estimation of "shadow thickness"
		float ShadowThickness = 0.0;
		{
			FRayDesc RayLocal;
			RayLocal.Origin = TranslatedWorldPosition + L * SkyLight.MaxShadowThickness;
			RayLocal.Direction = -L;
			RayLocal.TMin = 0.0;
			RayLocal.TMax = SkyLight.MaxShadowThickness;

			uint RayFlagsLocal = 0;
			const uint InstanceInclusionMaskLocal = RAY_TRACING_MASK_SHADOW;

#if !ENABLE_MATERIALS
			RayFlagsLocal |= RAY_FLAG_FORCE_OPAQUE;
#endif
#if !ENABLE_TWO_SIDED_GEOMETRY
			RayFlagsLocal |= RAY_FLAG_CULL_BACK_FACING_TRIANGLES;
#endif

			FMinimalPayload MinimalPayloadLocal = TraceVisibilityRay(
				TLAS,
				RayFlagsLocal,
				InstanceInclusionMaskLocal,
				RayLocal);

			float3 P = RayLocal.Origin + RayLocal.Direction * MinimalPayloadLocal.HitT;
			ShadowThickness = min(length(TranslatedWorldPosition - P), 1.0);
		}
		float3 ScatterThroughput = PI * KajiyaKayDiffuseAttenuation(GBufferData, L, V, N, ShadowThickness);
		float3 SkySHDiffuseIrradiance = GetSkySHDiffuse(L) * View.SkyLightColor.rgb;

		DiffuseExitantRadiance += SkySHDiffuseIrradiance * ScatterThroughput * (1.0 - AmbientOcclusion);
	}
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// special case of pixel coord == sample coord
void SkyLightEvaluate(
	in const uint2 SampleCoord,
	in const uint SamplesPerPixel,
	in const float3 TranslatedWorldPosition,
	in const float3 WorldNormal,
	in const float3 ViewDirection,
	in const FGBufferData GBufferData,
	in const RaytracingAccelerationStructure TLAS,
	in const bool bGBufferSampleOrigin,
	in const float DeviceZ,
	in const bool bDecoupleSampleGeneration,
	inout float3 ExitantRadiance,
	inout float3 DiffuseExitantRadiance,
	inout float AmbientOcclusion,
	inout float HitDistance)
{
	SkyLightEvaluate(
		SampleCoord,
		SampleCoord,
		SamplesPerPixel,
		TranslatedWorldPosition,
		WorldNormal,
		ViewDirection,
		GBufferData,
		TLAS,
		bGBufferSampleOrigin,
		DeviceZ,
		bDecoupleSampleGeneration,
		ExitantRadiance,
		DiffuseExitantRadiance,
		AmbientOcclusion,
		HitDistance);
}