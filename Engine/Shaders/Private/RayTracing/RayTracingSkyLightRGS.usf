// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTextureParameters.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"

#if USE_HAIR_LIGHTING
#include "../HairStrands/HairStrandsRaytracing.ush"
#endif

#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 1
#define SUBSTRATE_FASTPATH 0
#define SUBSTRATE_SINGLEPATH 0
#define SUBSTRATE_COMPLEXSPECIALPATH 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0		// SUBSTRATE_TODO: tile the pass to work with glint
#define SUBSTRATE_GLINTS_ALLOWED 0
#include "../Substrate/Substrate.ush"
#include "../Substrate/SubstrateEvaluation.ush"
#endif

#include "RayTracingSkyLightEvaluation.ush"
#include "RayTracingDeferredShadingCommon.ush"

uint UpscaleFactor;

RaytracingAccelerationStructure TLAS;

RWTexture2D<float4> RWSkyOcclusionMaskUAV;
RWTexture2D<float2> RWSkyOcclusionRayDistanceUAV;

RAY_TRACING_ENTRY_RAYGEN(SkyLightRGS)
{
	uint2 DispatchThreadId = DispatchRaysIndex().xy + View.ViewRectMin.xy;
	uint2 PixelCoord = GetPixelCoord(DispatchThreadId, UpscaleFactor);

	float DeviceZ = SceneDepthTexture.Load(int3(PixelCoord, 0)).r;
	float3 TranslatedWorldPosition;
	float3 CameraDirection;
	ReconstructTranslatedWorldPositionAndCameraDirectionFromDeviceZ(PixelCoord, DeviceZ, TranslatedWorldPosition, CameraDirection);

#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);

	// When Lumen is not used, only MaterialAO and ShadingID (see IsValid) are read, sourced form the single UINT read for the SubstratePixelHeader.
	const uint BSDFType = SubstratePixelHeader.SubstrateGetBSDFType();

	const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);

	float3 Albedo = 0;
	float3 DiffuseColor = 0;
	float3 WorldNormal = 0;
	Substrate_for(uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
	{
		float3 NullV = float3(0, 0, 1);
		float3 NullL = float3(0, 0, 1);
		FSubstrateBSDF CurrentBSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
		FSubstrateBSDFContext Context = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, SubstrateAddressing, NullV);

		if (SubstrateIsBSDFVisible(CurrentBSDF))
		{
			FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;	// Fake unused in SubstrateCreateBSDFContext when using Forward inline shading
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, NullV, NullL);
			FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);

			// Use LuminanceWeightV instead of LuminanceWeight(..) as we only need to weight these value with the view transmittance, not the light transmittance;
			const float3 Weight = CurrentBSDF.LuminanceWeightV;

			DiffuseColor  += Weight * SubstrateGetBSDFDiffuseColor(CurrentBSDF);
			WorldNormal   += Weight * SubstrateBSDFContext.N;
		}
	}

	FGBufferData GBufferData = (FGBufferData)0;
	GBufferData.ShadingModelID = BSDFType == SUBSTRATE_BSDF_TYPE_HAIR ? SHADINGMODELID_HAIR : SHADINGMODELID_DEFAULT_LIT;
	GBufferData.DiffuseColor = DiffuseColor;
	GBufferData.WorldNormal = normalize(WorldNormal);
	GBufferData.SpecularColor = 0;
	GBufferData.Roughness = 0.5f;
	// No anisotropy
	GBufferData.SelectiveOutputMask = 0;
	GBufferData.Anisotropy = 0.0f;

	WorldNormal = GBufferData.WorldNormal;
	Albedo = GBufferData.DiffuseColor;

#else // SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1

	// Get G-Buffer surface data
	float2 InvBufferSize = View.BufferSizeAndInvSize.zw;
	float2 UV = (float2(PixelCoord) + 0.5) * InvBufferSize;
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);

#if 0
	FGBufferData GBufferData = GetGBufferDataFromSceneTextures(UV);
#else
	//#dxr-todo: workaround for flickering. UE-87281
	FGBufferData GBufferData = GetGBufferDataFromSceneTexturesLoad(PixelCoord);
#endif

	float3 WorldNormal = GBufferData.WorldNormal;
	float3 Albedo = GBufferData.DiffuseColor;

	// Recalculate DiffuseColor if subsurface reverted the contribution within the G-Buffer
	if (UseSubsurfaceProfile(GBufferData.ShadingModelID))
	{
		Albedo = GBufferData.StoredBaseColor - GBufferData.StoredBaseColor * GBufferData.Metallic;
		GBufferData.DiffuseColor = Albedo;
	}

#endif // SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1

	// Mask out depth values that are infinitely far away
	bool IsFiniteDepth = DeviceZ > 0.0;
	bool bTraceRay = (
		IsFiniteDepth &&
		GBufferData.ShadingModelID != SHADINGMODELID_UNLIT);
	uint SamplesPerPixel = SkyLight.SamplesPerPixel;
	if (!bTraceRay)
	{
		SamplesPerPixel = 0;
	}

	// Evaluate the Sky Light at the surface point
	const bool bGBufferSampleOrigin = true;
	const bool bDecoupleSampleGeneration = DECOUPLE_SAMPLE_GENERATION != 0;
	float3 ExitantRadiance;
	float3 DiffuseExitantRadiance;
	float AmbientOcclusion;
	float HitDistance;

	SkyLightEvaluate(
		DispatchThreadId,
		PixelCoord,
		SamplesPerPixel,
		TranslatedWorldPosition,
		WorldNormal,
		CameraDirection,
		GBufferData,
		TLAS,
		bGBufferSampleOrigin,
		DeviceZ,
		bDecoupleSampleGeneration,
		ExitantRadiance,
		DiffuseExitantRadiance,
		AmbientOcclusion,
		HitDistance);

	// Pre-divide by albedo, to be recovered in compositing
	DiffuseExitantRadiance.r = Albedo.r > 0.0 ? DiffuseExitantRadiance.r / Albedo.r : DiffuseExitantRadiance.r;
	DiffuseExitantRadiance.g = Albedo.g > 0.0 ? DiffuseExitantRadiance.g / Albedo.g : DiffuseExitantRadiance.g;
	DiffuseExitantRadiance.b = Albedo.b > 0.0 ? DiffuseExitantRadiance.b / Albedo.b : DiffuseExitantRadiance.b;

	DiffuseExitantRadiance.rgb *= View.PreExposure;

	RWSkyOcclusionMaskUAV[DispatchThreadId] = float4(ClampToHalfFloatRange(DiffuseExitantRadiance.rgb), AmbientOcclusion);
	RWSkyOcclusionRayDistanceUAV[DispatchThreadId] = float2(HitDistance, SamplesPerPixel);

}