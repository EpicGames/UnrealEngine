// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../LightShaderParameters.ush"

bool GenerateSphereLightOcclusionRayWithAreaSampling(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPdf
)
{
	float4 Result = UniformSampleSphere(RandSample);
	float3 LightNormal = Result.xyz;
	float3 TranslatedLightPosition = LightParameters.TranslatedWorldPosition + LightNormal * LightParameters.SourceRadius;
	float3 LightDirection = TranslatedLightPosition - TranslatedWorldPosition;
	float RayLength = length(LightDirection);
	LightDirection /= RayLength;

	RayOrigin = TranslatedWorldPosition;
	RayDirection = LightDirection;
	RayTMin = 0.0;
	RayTMax = RayLength;

	float SolidAnglePdf = Result.w * saturate(dot(LightNormal, -LightDirection)) / (RayLength * RayLength);
	RayPdf = SolidAnglePdf;
	return true;
}

bool GenerateSphereLightOcclusionRayWithSolidAngleSampling(
	FLightShaderParameters LightParameters,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	float2 RandSample,
	out float3 RayOrigin,
	out float3 RayDirection,
	out float RayTMin,
	out float RayTMax,
	out float RayPdf
	)
{
	RayOrigin = 0.0;
	RayDirection = 0.0;
	RayTMin = 0.0;
	RayTMax = 0.0;
	RayPdf = 0.0;

	// Determine if shading point is contained within sphere light
	float3 LightDirection = LightParameters.TranslatedWorldPosition - TranslatedWorldPosition;
	float RayLength2 = dot(LightDirection, LightDirection);
	float Radius2 = LightParameters.SourceRadius * LightParameters.SourceRadius;

	BRANCH
	if (RayLength2 <= Radius2)
	{
		return GenerateSphereLightOcclusionRayWithAreaSampling(LightParameters, TranslatedWorldPosition, WorldNormal, RandSample,
			RayOrigin, RayDirection, RayTMin, RayTMax, RayPdf);
	}

	// Sample uniformly about a cone aligned with the z-axis
	float SinThetaMax2 = Radius2 / RayLength2;

#if 0
	float4 DirAndPdf = UniformSampleCone(RandSample, sqrt(1.0 - SinThetaMax2));
#elif 0
	float4 DirAndPdf = UniformSampleConeRobust(RandSample, SinThetaMax2);
#else
	float4 DirAndPdf = UniformSampleConeConcentricRobust(RandSample, SinThetaMax2);
#endif

	float CosTheta = DirAndPdf.z;
	float SinTheta2 = 1.0 - CosTheta * CosTheta;

	RayOrigin = TranslatedWorldPosition;
	// Project ray direction to world-space, such that z-axis aligns with LightDirection
	float RayLength = sqrt(RayLength2);
	LightDirection *= rcp(RayLength + 1e-4);
	RayDirection = TangentToWorld(DirAndPdf.xyz, LightDirection);

	RayTMin = 0.0;
	// Clip length to closest intersection with the sphere
	RayTMax = RayLength * (CosTheta - sqrt(max(SinThetaMax2 - SinTheta2, 0.0)));
	RayPdf = DirAndPdf.w;
	return true;
}