// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "RayTracingCommon.ush"

RaytracingAccelerationStructure  TLAS;
StructuredBuffer<FBasicRayData>  Rays;
RWStructuredBuffer<uint>         Output;

RAY_TRACING_ENTRY_RAYGEN(TestMainRGS)
{
	const uint RayIndex = DispatchRaysIndex().x;
	FBasicRayData InputRay = Rays[RayIndex];

	FRayDesc Ray;
	Ray.Origin = InputRay.Origin;
	Ray.Direction = InputRay.Direction;
	Ray.TMin = 0.0f;
	Ray.TMax = InputRay.TFar;

	uint RayFlags = 0
	              | RAY_FLAG_CULL_BACK_FACING_TRIANGLES      // use back face culling
	              | RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH // terminate immediately upon detecting primitive intersection
	              | RAY_FLAG_FORCE_OPAQUE                    // don't run anyhit shader
	              | RAY_FLAG_SKIP_CLOSEST_HIT_SHADER;        // don't run closesthit shader
	const uint InstanceInclusionMask = RAY_TRACING_MASK_ALL;

	FMinimalPayload MinimalPayload = TraceVisibilityRay(
		TLAS,
		RayFlags,
		InstanceInclusionMask,
		Ray);

	Output[RayIndex] = MinimalPayload.IsHit() ? ~0 : 0;
}
