// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#ifndef NUM_THREADS_X
#define NUM_THREADS_X 1024
#endif

#if PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER	

#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

ByteAddressBuffer IndexBuffer;
ByteAddressBuffer VertexBuffer;
uint VertexBufferStride;
uint VertexBufferOffsetInBytes;
uint IndexBufferOffsetInBytes;
uint IndexBufferStride;
uint NumPrimitives;
uint MaxVertices;

bool IsNaN(float F)
{
	uint U = asuint(F);
	uint ExponentMask = 0x7f800000;
	uint MantissaMask = 0x007fffff;
	return (U & ExponentMask) == ExponentMask 
		&& (U & MantissaMask) != 0;
}

// Returns true if all vertices are finite or if vertex X components are NaN. DXR inactive primitive rule applies:
// Triangles are considered "inactive" (but legal input to acceleration structure build) if the x component of each vertex is NaN.
void CheckTriangle(FTriangleBaseAttributes Tri, uint PrimitiveIndex)
{
	// All X-s are NaNs: triangle is legal and inactive
	if (IsNaN(Tri.LocalPositions[0].x) && IsNaN(Tri.LocalPositions[1].x) && IsNaN(Tri.LocalPositions[2].x))
	{
		return;
	}

	// As of 2022-03-28, some UE systems mark triangles as culled by only setting the provoking vertex to NaN
	// In practice, the drivers handle this, so we allow it for now.
	// Technically, this is a DXR spec violation.
	if (IsNaN(Tri.LocalPositions[0].x))
	{
		return;
	}
	
	// Active triangles must have non-NaN positions
	for (uint i=0; i<3; ++i)
	{
		if (IsNaN(Tri.LocalPositions[i].x) || IsNaN(Tri.LocalPositions[i].y) || IsNaN(Tri.LocalPositions[i].z))
		{
			uint AssertID = 0xF69EAF06; // Unique assert ID (TODO: add support for full human-readable assert strings)
			UEReportAssertWithPayload(AssertID,
			uint4(PrimitiveIndex, 
				asuint(Tri.LocalPositions[i].x), 
				asuint(Tri.LocalPositions[i].y), 
				asuint(Tri.LocalPositions[i].z)));
			return;
		}
	}
}

[numthreads(NUM_THREADS_X, 1, 1)]
void RayTracingValidateGeometryBuildParamsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint PrimitiveIndex = DispatchThreadId.x;

	if (PrimitiveIndex >= NumPrimitives)
	{
		return;
	}

	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(
		IndexBuffer, IndexBufferOffsetInBytes, IndexBufferStride,
		VertexBuffer, VertexBufferOffsetInBytes, VertexBufferStride,
		PrimitiveIndex);
	
	{
		// Check that index buffer value is within the expected range, based on VertexCount field of D3D12_RAYTRACING_GEOMETRY_TRIANGLES_DESC.
		// DXR spec for VertexCount: Number of vertices (positions) in VertexBuffer. If an index buffer is present, this must be at least the maximum index value in the index buffer + 1.
		for (uint i = 0; i < 3; ++i)
		{
			uint Index = Tri.Indices[i];
			if (Index >= MaxVertices)
			{
				uint AssertID = 0x5ECFB346; // Unique assert ID (TODO: add support for full human-readable assert strings)
				UEReportAssertWithPayload(AssertID, uint4(PrimitiveIndex, Index, MaxVertices, 0));
			}
		}
	}

	CheckTriangle(Tri, PrimitiveIndex);
}

uint NumInstances;
uint NumHitGroups;
ByteAddressBuffer InstanceBuffer;
uint InstanceBufferOffsetInBytes;
uint InstanceBufferStrideInBytes;

[numthreads(NUM_THREADS_X, 1, 1)]
void RayTracingValidateSceneBuildParamsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint InstanceIndex = DispatchThreadId.x;
	if (InstanceIndex >= NumInstances)
	{
		return;
	}

	uint InstanceAddress = InstanceBufferOffsetInBytes + InstanceBufferStrideInBytes * InstanceIndex;
	FPlatformRayTracingInstanceDescriptor InstanceDesc = InstanceBuffer.Load<FPlatformRayTracingInstanceDescriptor>(InstanceAddress);

	// Check that the instance matrix is finite (if instance culling is required, it should be done through null BLAS address)
	{
		for (uint i = 0; i < 3; ++i)
		{
			float4 Row = InstanceDesc.Transform[i];
			for (uint j = 0; j < 4; ++j)
			{
				float Value = Row[j];
				if (!IsFinite(Value))
				{
					UEReportAssertWithPayload(0x1C46B0FD, uint4(InstanceIndex, asuint(Value), 0, 0));
				}
			}
		}
	}

	// Check that flag bits belong to D3D12_RAYTRACING_INSTANCE_FLAGS entries (only first 4 bits can be set)
	{
		uint Flags = InstanceDesc.GetFlags();
		if ((Flags & 0xF0) != 0)
		{
			UEReportAssertWithPayload(0x07085473, uint4(InstanceIndex, Flags, 0, 0));
		}
	}

	// Check that instance contribution to SBT index is sane (will not cause OOB access)
	{
		uint Contribution = InstanceDesc.GetInstanceContributionToHitGroupIndex();
		if (Contribution > NumHitGroups)
		{
			UEReportAssertWithPayload(0xC5A7424A, uint4(InstanceIndex, Contribution, NumHitGroups, 0));
		}
	}
}

#else // PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER

[numthreads(NUM_THREADS_X, 1, 1)]
void RayTracingValidateGeometryBuildParamsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
}

[numthreads(NUM_THREADS_X, 1, 1)]
void RayTracingValidateSceneBuildParamsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
}

#endif // PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER

