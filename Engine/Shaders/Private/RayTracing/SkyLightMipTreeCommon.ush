// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	SkyLightMipTreeCommon.ush: Common utilities for MipTree-based SkyLight sampling
===============================================================================================*/

#include "MipTreeCommon.ush"

float3 GetTextureCubeVector(float3 TexelCoord, uint2 TextureRes)
{
	float3 Result;

	float2 UV = TexelCoord.xy / float2(TextureRes.xy);
	UV = UV * 2.0 - 1.0;

	uint Cubeface = TexelCoord.z;
	if (Cubeface == 0)
	{
		Result = float3(1, -UV.y, -UV.x);
	}
	else if (Cubeface == 1)
	{
		Result = float3(-1, -UV.y, UV.x);
	}
	else if (Cubeface == 2)
	{
		Result = float3(UV.x, 1, UV.y);
	}
	else if (Cubeface == 3)
	{
		Result = float3(UV.x, -1, -UV.y);
	}
	else if (Cubeface == 4)
	{
		Result = float3(UV.x, -UV.y, 1);
	}
	else if (Cubeface == 5)
	{
		Result = float3(-UV.x, -UV.y, -1);
	}

	return Result;
}

uint GetMaximumComponentIndex(float3 Input)
{
	float AbsX = abs(Input.x);
	float AbsY = abs(Input.y);
	float AbsZ = abs(Input.z);
	if (AbsX > AbsY)
	{
		return AbsX > AbsZ ? 0 : 2;
	}
	else
	{
		return AbsY > AbsZ ? 1 : 2;
	}
}

uint3 GetTextureCubeCoordinate(float3 WorldDirection, uint2 TextureRes)
{
	// Strongest coordinate determines cube face
	uint MaxComponent = GetMaximumComponentIndex(WorldDirection);
	WorldDirection /= abs(WorldDirection[MaxComponent]);
	float3 TextureCubeCoordinate;
	if (MaxComponent == 0)
	{
		if (WorldDirection[MaxComponent] > 0)
		{
			TextureCubeCoordinate = float3(-WorldDirection.z, -WorldDirection.y, 0);
		}
		else
		{
			TextureCubeCoordinate = float3(WorldDirection.z, -WorldDirection.y, 1);
		}
	}
	else if (MaxComponent == 1)
	{
		if (WorldDirection[MaxComponent] > 0)
		{
			TextureCubeCoordinate = float3(WorldDirection.x, WorldDirection.z, 2);
		}
		else
		{
			TextureCubeCoordinate = float3(WorldDirection.x, -WorldDirection.z, 3);
		}
	}
	else if (MaxComponent == 2)
	{
		if (WorldDirection[MaxComponent] > 0)
		{
			TextureCubeCoordinate = float3(WorldDirection.x, -WorldDirection.y, 4);
		}
		else
		{
			TextureCubeCoordinate = float3(-WorldDirection.x, -WorldDirection.y, 5);
		}
	}

	TextureCubeCoordinate.xy = (TextureCubeCoordinate.xy + 1.0) / 2.0;
	TextureCubeCoordinate.xy *= TextureRes;
	return TextureCubeCoordinate;
}

/**
* Cube texel solid angle computation based on formula from Manne Öhrström's
* thesis - "Spherical Harmonics, Precomputed Radiance Transfer and Realtime Radiosity in Computer Games".
*/
float AreaElement(float x, float y)
{
	// Equation 7.12
	return atan2(x * y, sqrt(x * x + y * y + 1));
}

float TexelCoordSolidAngle(uint2 TextureCoord, uint2 TextureRes)
{
	float2 InvResolution = 1.0f / (float)TextureRes;

	// Convert texture coordinate to [-1, 1] range, offset to texel center.
	float U = (2.0f * ((float)TextureCoord.x + 0.5f) * InvResolution.x) - 1.0f;
	float V = (2.0f * ((float)TextureCoord.y + 0.5f) * InvResolution.y) - 1.0f;

	// Projected area
	float x0 = U - InvResolution.x;
	float y0 = V - InvResolution.y;
	float x1 = U + InvResolution.x;
	float y1 = V + InvResolution.y;

	// Equation 7.12
	float SolidAngle = AreaElement(x0, y0) - AreaElement(x0, y1) - AreaElement(x1, y0) + AreaElement(x1, y1);
	return SolidAngle;
}

void BuildFaceCdf(uint MipCount, out float FaceCdf[7])
{
	uint BufferOffset = BufferOffsetAtPixel(uint2(0, 0), MipCount, SkyLight.MipDimensions.xy);

	FaceCdf[0] = 0.0;
	FaceCdf[1] = SkyLight.MipTreePosX[BufferOffset];
	FaceCdf[2] = FaceCdf[1] + SkyLight.MipTreeNegX[BufferOffset];
	FaceCdf[3] = FaceCdf[2] + SkyLight.MipTreePosY[BufferOffset];
	FaceCdf[4] = FaceCdf[3] + SkyLight.MipTreeNegY[BufferOffset];
	FaceCdf[5] = FaceCdf[4] + SkyLight.MipTreePosZ[BufferOffset];
	FaceCdf[6] = FaceCdf[5] + SkyLight.MipTreeNegZ[BufferOffset];
	float Sum = FaceCdf[6];

	for (uint Index = 1; Index < 6; ++Index)
	{
		FaceCdf[Index] /= Sum;
	}
	FaceCdf[6] = 1.0;
}

uint SampleFace(float FaceCdf[7], float RandSample, out float FacePdf)
{
	// Determine CDF entry
	uint FaceIndex = 1;
	while (RandSample > FaceCdf[FaceIndex] && FaceIndex < 7)
	{
		FaceIndex++;
	}

	// Calculate PDF and return entry
	FacePdf = FaceCdf[FaceIndex] - FaceCdf[FaceIndex - 1];

	// Cdf is incremented by 1
	return FaceIndex - 1;
}

uint SampleFace(int MipCount, float RandSample, out float FacePdf)
{
	float FaceCdf[7];
	BuildFaceCdf(MipCount, FaceCdf);
	return SampleFace(FaceCdf, RandSample, FacePdf);
}

float PdfFace(float FaceCdf[7], uint FaceIndex)
{
	// Cdf is offset by 1
	FaceIndex++;
	float Pdf = FaceCdf[FaceIndex] - FaceCdf[FaceIndex - 1];
	return Pdf;
}

float PdfFace(int MipCount, uint FaceIndex)
{
	float FaceCdf[7];
	BuildFaceCdf(MipCount, FaceCdf);
	return PdfFace(FaceCdf, FaceIndex);
}

float4 SampleMipTree(uint FaceIndex, uint4 BufferOffset)
{
	float4 Values;
	if (FaceIndex == 0)
	{
		Values.x = SkyLight.MipTreePosX[BufferOffset.x];
		Values.y = SkyLight.MipTreePosX[BufferOffset.y];
		Values.z = SkyLight.MipTreePosX[BufferOffset.z];
		Values.w = SkyLight.MipTreePosX[BufferOffset.w];
	}
	else if (FaceIndex == 1)
	{
		Values.x = SkyLight.MipTreeNegX[BufferOffset.x];
		Values.y = SkyLight.MipTreeNegX[BufferOffset.y];
		Values.z = SkyLight.MipTreeNegX[BufferOffset.z];
		Values.w = SkyLight.MipTreeNegX[BufferOffset.w];
	}
	else if (FaceIndex == 2)
	{
		Values.x = SkyLight.MipTreePosY[BufferOffset.x];
		Values.y = SkyLight.MipTreePosY[BufferOffset.y];
		Values.z = SkyLight.MipTreePosY[BufferOffset.z];
		Values.w = SkyLight.MipTreePosY[BufferOffset.w];
	}
	else if (FaceIndex == 3)
	{
		Values.x = SkyLight.MipTreeNegY[BufferOffset.x];
		Values.y = SkyLight.MipTreeNegY[BufferOffset.y];
		Values.z = SkyLight.MipTreeNegY[BufferOffset.z];
		Values.w = SkyLight.MipTreeNegY[BufferOffset.w];
	}
	else if (FaceIndex == 4)
	{
		Values.x = SkyLight.MipTreePosZ[BufferOffset.x];
		Values.y = SkyLight.MipTreePosZ[BufferOffset.y];
		Values.z = SkyLight.MipTreePosZ[BufferOffset.z];
		Values.w = SkyLight.MipTreePosZ[BufferOffset.w];
	}
	else
	{
		Values.x = SkyLight.MipTreeNegZ[BufferOffset.x];
		Values.y = SkyLight.MipTreeNegZ[BufferOffset.y];
		Values.z = SkyLight.MipTreeNegZ[BufferOffset.z];
		Values.w = SkyLight.MipTreeNegZ[BufferOffset.w];
	}

	return Values;
}

float PdfMipTree(uint3 TextureCoord, uint StopLevel)
{
	float MipPdf = 1.0;
	uint BufferOffset = BufferOffsetAtPixel(TextureCoord.xy, StopLevel, SkyLight.MipDimensions.xy);
	if (TextureCoord.z == 0)
	{
		MipPdf = SkyLight.MipTreePdfPosX[BufferOffset];
	}
	else if (TextureCoord.z == 1)
	{
		MipPdf = SkyLight.MipTreePdfNegX[BufferOffset];
	}
	else if (TextureCoord.z == 2)
	{
		MipPdf = SkyLight.MipTreePdfPosY[BufferOffset];
	}
	else if (TextureCoord.z == 3)
	{
		MipPdf = SkyLight.MipTreePdfNegY[BufferOffset];
	}
	else if (TextureCoord.z == 4)
	{
		MipPdf = SkyLight.MipTreePdfPosZ[BufferOffset];
	}
	else // if (TextureCoord.z == 5)
	{
		MipPdf = SkyLight.MipTreePdfNegZ[BufferOffset];
	}
	return MipPdf;
}


