// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FSkyLightVisibilityRays
{
	float4 DirectionAndPdf;
};

uint GetSkyLightVisibilityRayIndex(
	in uint2 PixelCoord,
	in uint SampleIndex,
	in uint2 SkyLightVisibilityRaysDimensions)
{
	// Calculate the index into the Sky Light Visibility Ray buffer from the given Pixel Coordinate and Sample Index
	const uint LinearIndex = (PixelCoord.y * SkyLightVisibilityRaysDimensions.x + PixelCoord.x) * SkyLight.SamplesPerPixel + SampleIndex;

	return LinearIndex;
}

uint GetSkyLightVisibilityRayTiledIndex(
	in uint2 PixelCoord,
	in uint SampleIndex,
	in uint2 SkyLightVisibilityRaysDimensions)
{
	// Calculate the index into the Sky Light Visibility Ray buffer from the given Pixel Coordinate and Sample Index
	// Note that for reading PixelCoord may be much larger than the Sky Light Visibility Rays dimensions and is applied in a repeating tile pattern, hence the modulus by the dimensions
	const uint2 RayCoord = PixelCoord % SkyLightVisibilityRaysDimensions.xy;
	const uint RayCoordLinearIndex = (RayCoord.y * SkyLightVisibilityRaysDimensions.x + RayCoord.x) * SkyLight.SamplesPerPixel + SampleIndex;

	return RayCoordLinearIndex;
}
