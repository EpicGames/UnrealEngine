// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef PLATFORM_SUPPORTS_INLINE_RAY_TRACING
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING 0
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#if !COMPUTESHADER
#error Inline ray tracing is only supported in compute shaders
#endif // !COMPUTESHADER

#ifndef ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
#	define ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE 0
#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE

#ifndef ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS
#	define ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS 0
#endif // ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS

#if OVERRIDE_TRACERAYINLINE_USH
#include "/Platform/Private/TraceRayInline.ush"

#else // OVERRIDE_TRACERAYINLINE_USH

#include "/Engine/Private/RayTracing/TraceRayInlineCommon.ush"

#if (COMPILER_VULKAN || PLATFORM_USE_TRACE_RAY_INLINE_VULKAN_USH)
#include "/Engine/Private/RayTracing/TraceRayInlineVulkan.ush"
#elif COMPILER_METAL
#include "/Engine/Private/RayTracing/TraceRayInlineMetal.ush"
#elif SM6_PROFILE || SM5_PROFILE
#include "/Engine/Private/RayTracing/TraceRayInlineD3D12.ush"
#endif // COMPILER_VULKAN

#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
#	include "/Engine/Public/NvidiaExtensionsCommon.ush"
#endif

#ifndef PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS
#	define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS 0
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS

#if !PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS
// placeholder since platform is not expected to fill WorldGeometryNormal
#define GetInlineRayTracingWorldGeometryNormal(RayQ, Mat, Committed) RayQ.WorldRayDirection()
#endif

#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS 0
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX 1

template<typename TTraceRayInlineCallback>
FTraceRayInlineResult TraceRayInlineWithCallback(RaytracingAccelerationStructure AccelerationStructure,
	uint RayFlags,
	uint InstanceInclusionMask,
	RayDesc Ray,
	FTraceRayInlineContext TraceRayInlineContext,
	inout TTraceRayInlineCallback Callback)
{
	FTraceRayInlineResult TraceRayResult = (FTraceRayInlineResult)0;
	// Compiler has issues with the line below
	//TraceRayResult.SetMiss();
	TraceRayResult.HitT = -1;
	TraceRayResult.bIsCompleted = true;

#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	RayQuery<RAY_FLAG_NONE> RayQ;
#else
	RayQuery<RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES> RayQ;
#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE

	RayQ.TraceRayInline(AccelerationStructure, RayFlags, InstanceInclusionMask, Ray);

	while (RayQ.Proceed())
	{
		switch (RayQ.CandidateType())
		{
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
		case CANDIDATE_PROCEDURAL_PRIMITIVE:
		{
			if (TraceRayInlineContext.bProcedural)
			{
				FTraceRayInlineProceduralIntersectionParameters CandidateProceduralIntersection;
				CandidateProceduralIntersection.InstanceID = RayQ.CandidateInstanceID();
				CandidateProceduralIntersection.InstanceIndex = RayQ.CandidateInstanceIndex();
				CandidateProceduralIntersection.PrimitiveIndex = RayQ.CandidatePrimitiveIndex();
				CandidateProceduralIntersection.GeometryIndex = RayQ.CandidateGeometryIndex();
				CandidateProceduralIntersection.InstanceContributionToHitGroupIndex = RayQ.CandidateInstanceContributionToHitGroupIndex();

				const float3 RayOrg = RayQ.CandidateObjectRayOrigin();
				const float3 RayDir = RayQ.CandidateObjectRayDirection();
				const float TMin = RayQ.RayTMin();
				const float TMax = RayQ.CommittedRayT();

				float IntersectionT = TMax;

				if (Callback.OnProceduralPrimitive(RayOrg, RayDir, TMin, IntersectionT, CandidateProceduralIntersection))
				{
					RayQ.CommitProceduralPrimitiveHit(IntersectionT);
				}
			}					
			break;
		}
#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
		case CANDIDATE_NON_OPAQUE_TRIANGLE:
		{
			FTraceRayInlineResult CandidateTraceRayResult;
			CandidateTraceRayResult.HitT = RayQ.CandidateTriangleRayT();
			CandidateTraceRayResult.InstanceID = RayQ.CandidateInstanceID();
			CandidateTraceRayResult.InstanceIndex = RayQ.CandidateInstanceIndex();
			CandidateTraceRayResult.PrimitiveIndex = RayQ.CandidatePrimitiveIndex();
			CandidateTraceRayResult.GeometryIndex = RayQ.CandidateGeometryIndex();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
			CandidateTraceRayResult.ClusterID = TraceRayInlineContext.bOutputClusterID ? NvRtGetCandidateClusterID(RayQ) : 0;
#endif
			CandidateTraceRayResult.Barycentrics = RayQ.CandidateTriangleBarycentrics();
			CandidateTraceRayResult.bIsFrontFace = RayQ.CandidateTriangleFrontFace();
			CandidateTraceRayResult.InstanceContributionToHitGroupIndex = RayQ.CandidateInstanceContributionToHitGroupIndex();

			float3x4 WorldToObject3x4 = RayQ.CandidateWorldToObject3x4();
			CandidateTraceRayResult.WorldToObject3x4 = WorldToObject3x4;

			CandidateTraceRayResult.WorldGeometryNormal = GetInlineRayTracingWorldGeometryNormal(RayQ, TraceRayResult.WorldToObject3x4, false);

			CandidateTraceRayResult.Bookmark[0] = asuint(TraceRayResult.HitT);
			CandidateTraceRayResult.Bookmark[1] = asuint(TraceRayResult.HitT);
			if (Callback.OnAnyHit(RayQ.CandidateObjectRayOrigin(), RayQ.CandidateObjectRayDirection(), CandidateTraceRayResult))
			{
				RayQ.CommitNonOpaqueTriangleHit();
			}
			break;
		}
		}
	}

	switch (RayQ.CommittedStatus())
	{
	case COMMITTED_TRIANGLE_HIT:
	{
		TraceRayResult.HitT = RayQ.CommittedRayT();
		TraceRayResult.InstanceID = RayQ.CommittedInstanceID();
		TraceRayResult.InstanceIndex = RayQ.CommittedInstanceIndex();
		TraceRayResult.PrimitiveIndex = RayQ.CommittedPrimitiveIndex();
		TraceRayResult.GeometryIndex = RayQ.CommittedGeometryIndex();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
		TraceRayResult.ClusterID = TraceRayInlineContext.bOutputClusterID ? NvRtGetCommittedClusterID(RayQ) : 0;
#endif
		TraceRayResult.Barycentrics = RayQ.CommittedTriangleBarycentrics();
		TraceRayResult.bIsFrontFace = RayQ.CommittedTriangleFrontFace();
		TraceRayResult.InstanceContributionToHitGroupIndex = RayQ.CommittedInstanceContributionToHitGroupIndex();

		TraceRayResult.Bookmark[0] = asuint(TraceRayResult.HitT);
		TraceRayResult.Bookmark[1] = asuint(TraceRayResult.HitT);

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX
		TraceRayResult.WorldToObject3x4 = RayQ.CommittedWorldToObject3x4();
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX

		TraceRayResult.WorldGeometryNormal = GetInlineRayTracingWorldGeometryNormal(RayQ, TraceRayResult.WorldToObject3x4, true);

#if ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS
		// statistics?
#endif // ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS
		break;
	}
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	case COMMITTED_PROCEDURAL_PRIMITIVE_HIT:
	{
		TraceRayResult.bIsProcedural = true;
		TraceRayResult.HitT = RayQ.CommittedRayT();
		TraceRayResult.InstanceID = RayQ.CommittedInstanceID();
		TraceRayResult.InstanceIndex = RayQ.CommittedInstanceIndex();
		TraceRayResult.PrimitiveIndex = RayQ.CommittedPrimitiveIndex();
		TraceRayResult.GeometryIndex = RayQ.CommittedGeometryIndex();
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
		TraceRayResult.ClusterID = TraceRayInlineContext.bOutputClusterID ? NvRtGetCommittedClusterID(RayQ) : 0;
#endif
		TraceRayResult.Barycentrics = float2(0.0f, 0.0f); // ignored for procedural
		TraceRayResult.bIsFrontFace = true; // ignored for procedural
		TraceRayResult.InstanceContributionToHitGroupIndex = RayQ.CommittedInstanceContributionToHitGroupIndex();

		TraceRayResult.Bookmark[0] = asuint(TraceRayResult.HitT);
		TraceRayResult.Bookmark[1] = asuint(TraceRayResult.HitT);

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX
		TraceRayResult.WorldToObject3x4 = RayQ.CommittedWorldToObject3x4();
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX

		TraceRayResult.WorldGeometryNormal = GetInlineRayTracingWorldGeometryNormal(RayQ, TraceRayResult.WorldToObject3x4, true);

		break;
	}
#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	}

	return TraceRayResult;
}

FTraceRayInlineResult TraceRayInline(RaytracingAccelerationStructure AccelerationStructure,
	uint RayFlags,
	uint InstanceInclusionMask,
	RayDesc InRay,
	FTraceRayInlineContext TraceRayInlineContext)
{
	FDefaultTraceRayInlineCallback DefaultCallback;
	RayFlags |= RAY_FLAG_FORCE_OPAQUE;
	return TraceRayInlineWithCallback(AccelerationStructure, RayFlags, InstanceInclusionMask, InRay, TraceRayInlineContext, DefaultCallback);
}

#endif // OVERRIDE_TRACERAYINLINE_USH

#ifndef PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS 0
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS

#ifndef PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES 0
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES

#ifndef PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX 0
#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX

#ifndef FRayTracingSceneMetadataRecord
// Not used, but must be a valid structure declaration as it may appear in interfaces, resource declarations, etc.
struct FRayTracingSceneMetadataRecord { uint Dummy; };
#endif // FRayTracingSceneMetadataRecord

#endif // PLATFORM_SUPPORTS_INLINE_RAY_TRACING
