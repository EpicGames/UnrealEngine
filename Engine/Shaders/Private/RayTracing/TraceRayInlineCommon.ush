// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Update this GUID to invalidate inline RayTracing shaders and force a shader recompilation.
#pragma message("UESHADERMETADATA_VERSION 06A6E5F1-49BF-4FF3-9D90-4B501E7E5FC4")

struct FTraceRayInlineTraversalStatistics
{
	uint NodeIntersectionCount;
	uint ClusterIntersectionCount;
	uint TriangleIntersectionCount;
	uint IterationCount;

	uint WaveNodeIntersectionCount;
	uint WaveTriangleIntersectionCount;

	uint RayCount;
	uint WavefrontCount;

	uint ActiveLaneCount;
	uint TotalLaneCount;
};

static FTraceRayInlineTraversalStatistics TraceRayInlineTraversalStatistics_Sample;
RWStructuredBuffer<FTraceRayInlineTraversalStatistics> TraceRayInlineTraversalStatistics_Accumulator;

void TraceRayInlineAccumulateStatistics()
{
#if ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS
	if (WaveIsFirstLane())
	{
		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].RayCount, TraceRayInlineTraversalStatistics_Sample.RayCount);
		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].WavefrontCount, TraceRayInlineTraversalStatistics_Sample.WavefrontCount);

		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].ActiveLaneCount, TraceRayInlineTraversalStatistics_Sample.ActiveLaneCount);
		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].TotalLaneCount, TraceRayInlineTraversalStatistics_Sample.TotalLaneCount);

		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].IterationCount, TraceRayInlineTraversalStatistics_Sample.IterationCount);

		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].WaveNodeIntersectionCount, TraceRayInlineTraversalStatistics_Sample.WaveNodeIntersectionCount);
		InterlockedAdd(TraceRayInlineTraversalStatistics_Accumulator[0].WaveTriangleIntersectionCount, TraceRayInlineTraversalStatistics_Sample.WaveTriangleIntersectionCount);
	}
#endif
}

struct FTraceRayInlineResult
{
	float HitT;
	float2 Barycentrics; // Only valid if bIsProcedural is false
	uint InstanceIndex;
	uint InstanceID;
	uint InstanceContributionToHitGroupIndex;
	uint GeometryIndex;
	uint PrimitiveIndex;
#if PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	uint ClusterID;
#endif
	bool bIsFrontFace; // Only valid if bIsProcedural is false

	uint2 Bookmark;

	// Only valid when PLATFORM_SUPPORTS_INLINE_RAY_TRACING_INSTANCE_MATRIX is 1
	float3x4 WorldToObject3x4;

	// Only valid when PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS is 1
	float3 WorldGeometryNormal;

	// Can be false only if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS is 1
	bool bIsCompleted;

	// Can only be true if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE is 1
	bool bIsProcedural;

	// Only valid if shader was compiled with ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS
	FTraceRayInlineTraversalStatistics TraversalStatistics;

	bool IsMiss() { return HitT < 0; }
	bool IsHit() { return !IsMiss(); }

	void SetMiss() { HitT = -1; }
};

struct FTraceRayInlineContext
{
	// Only used if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_MAX_ITERATIONS is 1
	uint MaxIterations;

	// Only used if shader was compiled with ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	bool bProcedural;

	// Only used if shader was compiled with PLATFORM_SUPPORTS_RAYTRACING_CLUSTER_OPS
	bool bOutputClusterID;
};

struct FTraceRayInlineProceduralIntersectionParameters
{
	uint InstanceIndex;
	uint InstanceID;
	uint InstanceContributionToHitGroupIndex;
	uint GeometryIndex;
	uint PrimitiveIndex;
};

// Default callback structure for use with TraceRayInline.
// If custom functions are needed the new structure should derive from this structure.
struct FDefaultTraceRayInlineCallback
{	
	// Returning true means to accept the hit and false means to continue search (default accept)
	bool OnAnyHit(float3 ObjectRayOrigin, float3 ObjectRayDirection, FTraceRayInlineResult Hit)
	{ 
		return true; 
	}

	// Returning true means accept the procedural hit and false means to skip it (default skip)
	bool OnProceduralPrimitive(float3 ObjectRayOrigin, float3 ObjectRayDirection, float RayTMin, inout float RayTCurrent, FTraceRayInlineProceduralIntersectionParameters ProceduralIntersectionParameters)
	{
		return false;
	}
};

FTraceRayInlineContext CreateTraceRayInlineContext()
{
	FTraceRayInlineContext Context;
	Context.bProcedural = false;
	Context.bOutputClusterID = false;
	Context.MaxIterations = 0xFFFFFFFF;
	return Context;
}
