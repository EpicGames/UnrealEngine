// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	TraceRayInlineD3D12.ush: D3D12-specific inline ray tracing functionality,
	such as utility functions for accessing triangle indices and vertices.
==============================================================================*/

#pragma once

#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingBuiltInResources.h"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

#if COMPILER_DXC && PLATFORM_SUPPORTS_INLINE_RAY_TRACING && UE_BINDLESS_ENABLED

#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES 1

struct FD3DHitGroupSystemParametersWithPadding
{
	FHitGroupSystemRootConstants RootConstants;

	uint BindlessHitGroupSystemIndexBuffer;
	uint BindlessHitGroupSystemVertexBuffer;

	// Add padding because c++ side has union for GPU address of index/vertex buffer data for non bindless code path which are uint64 each
	uint Padding0;
	uint Padding1;
};

// Create a generic alias for the Vulkan-specific metadata type
#define FRayTracingSceneMetadataRecord FD3DHitGroupSystemParametersWithPadding

uint GetInlineRayTracingHitGroupFirstPrimitive(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FRayTracingSceneMetadataRecord Params = RayTracingSceneMetadata[RecordIndex];

	return Params.RootConstants.FirstPrimitive;
}

FTriangleBaseAttributes LoadInlineRayTracingTriangleAttributes(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex,
	uint PrimitiveIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FRayTracingSceneMetadataRecord Params = RayTracingSceneMetadata[RecordIndex];
		
	const uint IndexBufferOffsetInBytes = Params.RootConstants.IndexBufferOffsetInBytes;
	const uint IndexBufferStride = Params.RootConstants.GetIndexStride();
	const uint VertexStride = Params.RootConstants.GetVertexStride();
	const uint VertexBufferOffsetInBytes = 0; // Base offset is already applied when the buffer is bound to the hit group
	
	const uint IndexBufferHandle = Params.BindlessHitGroupSystemIndexBuffer;
	const uint VertexBufferHandle = Params.BindlessHitGroupSystemVertexBuffer;

	ByteAddressBuffer IndexBuffer = ResourceDescriptorHeap[NonUniformResourceIndex(IndexBufferHandle)];
	ByteAddressBuffer VertexBuffer = ResourceDescriptorHeap[NonUniformResourceIndex(VertexBufferHandle)];	

	return LoadTriangleBaseAttributes(IndexBuffer, IndexBufferOffsetInBytes, IndexBufferStride, VertexBuffer, VertexBufferOffsetInBytes, VertexStride, PrimitiveIndex);
}

#endif // COMPILER_DXC && PLATFORM_SUPPORTS_INLINE_RAY_TRACING && UE_BINDLESS_ENABLED
