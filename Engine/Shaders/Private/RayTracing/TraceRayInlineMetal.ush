// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	TraceRayInlineMetal.ush: Metal-specific inline ray tracing functionality,
	such as utility functions for accessing triangle indices and vertices.
==============================================================================*/

#pragma once

#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Shared/RayTracingBuiltInResources.h"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"
#include "/Engine/Private/RayTracing/RayTracingHitGroupCommon.ush"

#if COMPILER_DXC && PLATFORM_SUPPORTS_INLINE_RAY_TRACING && UE_BINDLESS_ENABLED

#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES 1

#define FRayTracingSceneMetadataRecord FMetalHitGroupSystemParameters

uint GetInlineRayTracingHitGroupFirstPrimitive(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FRayTracingSceneMetadataRecord Params = RayTracingSceneMetadata[RecordIndex];

	return Params.RootConstants.FirstPrimitive;
}

FTriangleBaseAttributes LoadInlineRayTracingTriangleAttributes(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex,
	uint PrimitiveIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FRayTracingSceneMetadataRecord Params = RayTracingSceneMetadata[RecordIndex];
		
	const uint IndexBufferOffsetInBytes = Params.RootConstants.IndexBufferOffsetInBytes;
	const uint IndexBufferStride = Params.RootConstants.GetIndexStride();
	const uint VertexStride = Params.RootConstants.GetVertexStride();
	const uint VertexBufferOffsetInBytes = 0; // Base offset is already applied when the buffer is bound to the hit group
	
	const uint IndexBufferHandle = Params.BindlessHitGroupSystemIndexBuffer;
	const uint VertexBufferHandle = Params.BindlessHitGroupSystemVertexBuffer;

	ByteAddressBuffer IndexBuffer = ResourceDescriptorHeap[NonUniformResourceIndex(IndexBufferHandle)];
	ByteAddressBuffer VertexBuffer = ResourceDescriptorHeap[NonUniformResourceIndex(VertexBufferHandle)];	
		
	return LoadTriangleBaseAttributes(IndexBuffer, IndexBufferOffsetInBytes, IndexBufferStride, VertexBuffer, VertexBufferOffsetInBytes, VertexStride, PrimitiveIndex);
}

#endif // COMPILER_DXC && PLATFORM_SUPPORTS_INLINE_RAY_TRACING && UE_BINDLESS_ENABLED
