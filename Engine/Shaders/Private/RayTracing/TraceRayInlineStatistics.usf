// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShaderPrint.ush"
#include "TraceRayInlineCommon.ush"

StructuredBuffer<FTraceRayInlineTraversalStatistics> TraversalStatistics;

[numthreads(1, 1, 1)]
void TraceRayInlinePrintStatisticsCS()
{
	const float TopMargin = 0.05f;
	const float HeadlineX = 0.75f;
	const float ItemX = 0.76f;

	FShaderPrintContext Context = InitShaderPrintContext(true, float2(HeadlineX, TopMargin));

	const uint PassIndex = 0; 
	const float RayCount = TraversalStatistics[PassIndex].RayCount;
	const float WavefrontCount = TraversalStatistics[PassIndex].WavefrontCount;

	Print(Context, TEXT("Pass Traversal Statistics"), FontOrange);
	Newline(Context);

	Print(Context, TEXT("  Ray Count            "), FontYellow);
	Print(Context, TraversalStatistics[PassIndex].RayCount);
	Newline(Context);

	Print(Context, TEXT("  Wavefront Count      "), FontYellow);
	Print(Context, TraversalStatistics[PassIndex].WavefrontCount);
	Newline(Context);

	// Node Intersection
	Print(Context, TEXT("  Tri/Ray              "), FontYellow);
	float TriangleRay = TraversalStatistics[PassIndex].WaveTriangleIntersectionCount / RayCount;
	Print(Context, TriangleRay);
	Newline(Context);

	Print(Context, TEXT("  Node/Ray             "), FontYellow);
	float NodeRay = TraversalStatistics[PassIndex].WaveNodeIntersectionCount / RayCount;
	Print(Context, NodeRay);
	Newline(Context);

	Print(Context, TEXT("  Iteration/Wave       "), FontYellow);
	float IterationWave = TraversalStatistics[PassIndex].IterationCount / WavefrontCount;
	Print(Context, IterationWave);
	Newline(Context);

	Print(Context, TEXT("  Iteration/Ray        "), FontYellow);
	float IterationRay = TriangleRay + NodeRay;
	Print(Context, IterationRay);
	Newline(Context);

	Print(Context, TEXT("  Iteration Divergence "), FontYellow);
	Print(Context, IterationWave / IterationRay);
	Newline(Context);

	Print(Context, TEXT("  Occupancy (%)        "), FontYellow);
	float Occupancy = 100.0 * TraversalStatistics[PassIndex].ActiveLaneCount / TraversalStatistics[PassIndex].TotalLaneCount;
	Print(Context, Occupancy);
	Newline(Context);
}
