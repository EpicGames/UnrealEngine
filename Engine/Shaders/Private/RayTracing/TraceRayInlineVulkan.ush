// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	TraceRayInlineVulkan.ush: Vulkan-specific inline ray tracing functionality,
	such as utility functions for accessing triangle indices and vertices.
==============================================================================*/

#pragma once

#include "/Engine/Shared/RayTracingDefinitions.h"
#include "/Engine/Private/RayTracing/RayTracingCommon.ush"

#if COMPILER_DXC && (COMPILER_VULKAN || PLATFORM_USE_TRACE_RAY_INLINE_VULKAN_USH) && PLATFORM_SUPPORTS_INLINE_RAY_TRACING

#ifndef VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH
#define VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH 0
#endif

// Metadata buffer is not allocated when position_fetch is available
#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES !VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

#if EMULATE_VKDEVICEADRESS
inline uint2 AddEmulated64(uint2 a, uint b)
{
    uint2 c;
    c.x = a.x + b;
    c.y = a.y + (c.x < a.x); // Add with carry.
    return c;
}

#define VK_ADDR_TYPE uint2
#define VK_RAW_BUFFER_LOAD(Type, Addr, Offs) vk::RawBufferLoad2<Type>(AddEmulated64(Addr,Offs))
#else
#define VK_ADDR_TYPE uint64_t
#define VK_RAW_BUFFER_LOAD(Type, Addr, Offs) vk::RawBufferLoad<Type>(Addr + Offs)
#endif 

uint3 VulkanLoadIndices16Bit(VK_ADDR_TYPE SourceBufferBaseAddress, uint OffsetInBytes)
{
	uint3 Result;

	// ByteAddressBuffer loads must be aligned to DWORD boundary.
	// We can load 2 DWORDs (4 SHORTs) at a time and extract 3 SHORTs (first 3 or second 3).
	uint AlignedOffsetInBytes = OffsetInBytes & (~3);
	const uint2 PackedIndices = VK_RAW_BUFFER_LOAD(uint2, SourceBufferBaseAddress, AlignedOffsetInBytes);

	if (AlignedOffsetInBytes == OffsetInBytes)
	{
		// Extract first 3 SHORTs from 2 DWORDs
		Result[0] = PackedIndices[0] & 0xffff;
		Result[1] = PackedIndices[0] >> 16;
		Result[2] = PackedIndices[1] & 0xffff;
	}
	else
	{
		// Extract second 3 SHORTs from 2 DWORDs
		Result[0] = PackedIndices[0] >> 16;
		Result[1] = PackedIndices[1] & 0xffff;
		Result[2] = PackedIndices[1] >> 16;
	}

	return Result;
}

uint3 VulkanLoadIndices32Bit(VK_ADDR_TYPE SourceBufferBaseAddress, uint OffsetInBytes)
{
	return VK_RAW_BUFFER_LOAD(uint3, SourceBufferBaseAddress, OffsetInBytes);
}

float3 VulkanLoadVertexPositionFloat3(VK_ADDR_TYPE SourceBufferBaseAddress, uint Index, uint StrideInBytes)
{
	uint OffsetInBytes = Index * StrideInBytes;
	return asfloat(VK_RAW_BUFFER_LOAD(uint3, SourceBufferBaseAddress, OffsetInBytes));
}

#if VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

#define PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS 1

[[vk::ext_capability(5391 /* RayQueryPositionFetchKHR */)]]
[[vk::ext_extension("SPV_KHR_ray_tracing_position_fetch")]]
[[vk::ext_instruction(5340 /* OpRayQueryGetIntersectionTriangleVertexPositionsKHR */)]]
float3 RayQueryGetIntersectionTriangleVertexPositionsKHR(
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	[[vk::ext_reference]] RayQuery<RAY_FLAG_NONE> RayQ,
#else
	[[vk::ext_reference]] RayQuery<RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES> RayQ,
#endif // ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
int committed)[3];

float3 GetInlineRayTracingWorldGeometryNormal(
#if ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE
	RayQuery<RAY_FLAG_NONE> RayQ,
#else
	RayQuery<RAY_FLAG_SKIP_PROCEDURAL_PRIMITIVES> RayQ,
#endif
	float3x4 WorldToObject3x4,
	bool Committed)
{
	FTriangleBaseAttributes Result = (FTriangleBaseAttributes)0;

	float3 LocalNormal;
	if (Committed)
	{
		float3 LocalPositions[3] = RayQueryGetIntersectionTriangleVertexPositionsKHR(RayQ, 1 /* RayQueryCommittedIntersectionKHR */);
		LocalNormal = cross(LocalPositions[2] - LocalPositions[0], LocalPositions[1] - LocalPositions[0]);
	}
	else
	{
		float3 LocalPositions[3] = RayQueryGetIntersectionTriangleVertexPositionsKHR(RayQ, 0 /* RayQueryCandidateIntersectionKHR */);
		LocalNormal = cross(LocalPositions[2] - LocalPositions[0], LocalPositions[1] - LocalPositions[0]);
	}

	float3 WorldNormal = normalize(mul(LocalNormal, (float3x3)WorldToObject3x4));
	return WorldNormal;
}

#else // VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

// Counterpart of FVulkanRayTracingGeometryParameters in VulkanRayTracing.cpp
struct FVulkanRayTracingGeometryParameters
{
	uint Config;
	uint IndexBufferOffsetInBytes;
	VK_ADDR_TYPE IndexBuffer;
	VK_ADDR_TYPE VertexBuffer;
};

// Create a generic alias for the Vulkan-specific metadata type
#define FRayTracingSceneMetadataRecord FVulkanRayTracingGeometryParameters

uint GetInlineRayTracingHitGroupFirstPrimitive(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FRayTracingSceneMetadataRecord Params = RayTracingSceneMetadata[RecordIndex];

	return 0; // TODO
}

FTriangleBaseAttributes LoadInlineRayTracingTriangleAttributes(
	StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata,
	uint InstanceContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint GeometryIndex,
	uint PrimitiveIndex)
{
	uint RecordIndex = InstanceContributionToHitGroupIndex / MultiplierForGeometryContributionToHitGroupIndex + GeometryIndex;
	FVulkanRayTracingGeometryParameters Params = RayTracingSceneMetadata[RecordIndex];

	FTriangleBaseAttributes Result = (FTriangleBaseAttributes)0;

	// Fetch vertex indices and positions, then compute local space normal and transform it to world space

	const uint IndexBufferOffsetInBytes = Params.IndexBufferOffsetInBytes;
	const uint IndexBufferStride = Params.Config & 0xFF;
	const uint BaseIndex = PrimitiveIndex * 3;

	if (IndexBufferStride == 0)
	{
		// Non-indexed geometry (implicit triangle list indices)
		Result.Indices = uint3(BaseIndex, BaseIndex + 1, BaseIndex + 2);
	}
	else if (IndexBufferStride == 2)
	{
		Result.Indices = VulkanLoadIndices16Bit(Params.IndexBuffer, IndexBufferOffsetInBytes + BaseIndex * IndexBufferStride);
	}
	else
	{
		Result.Indices = VulkanLoadIndices32Bit(Params.IndexBuffer, IndexBufferOffsetInBytes + BaseIndex * IndexBufferStride);
	}

	// Fetch vertex positions (in local space)
	const uint VertexStride = (Params.Config >> 8) & 0xFF;

	// #dxr_todo: UE-72160 handle various vertex formats, for now only supporting float3
	Result.LocalPositions[0] = VulkanLoadVertexPositionFloat3(Params.VertexBuffer, Result.Indices[0], VertexStride);
	Result.LocalPositions[1] = VulkanLoadVertexPositionFloat3(Params.VertexBuffer, Result.Indices[1], VertexStride);
	Result.LocalPositions[2] = VulkanLoadVertexPositionFloat3(Params.VertexBuffer, Result.Indices[2], VertexStride);

	return Result;
}

#endif // VULKAN_SUPPORTS_RAY_TRACING_POSITION_FETCH

#endif // COMPILER_DXC && (COMPILER_VULKAN || PLATFORM_USE_TRACE_RAY_INLINE_VULKAN_USH) && PLATFORM_SUPPORTS_INLINE_RAY_TRACING
