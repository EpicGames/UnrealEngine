// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VFXTraceRay.ush: VFX Specific ray tracing utilities.
=============================================================================*/

#pragma once

#ifndef VFX_TRACE_RAY_USH_INCLUDED
#define VFX_TRACE_RAY_USH_INCLUDED

#include "RayTracingCommon.ush"
#include "RayTracingHitGroupCommon.ush"

#if COMPUTESHADER
#include "TraceRayInline.ush"

struct FVFXTracePayload
{
	float HitT;
	uint GPUSceneInstanceId;
	float2 Barycentrics;
	float3 TranslatedWorldPosition;
	float3 WorldNormal;

	bool IsMiss() { return HitT < 0; }
	bool IsHit() { return !IsMiss(); }

	void SetMiss() { HitT = -1; }
};

#endif

#ifdef OVERRIDE_VFXTRACERAY_USH
#include "/Platform/Private/VFXTraceRay.ush"
#else // OVERRIDE_VFXTRACERAY_USH

void InitVFXTracePayloadFromHitShader(FRayTracingIntersectionAttributes Attributes, out FVFXTracePayload OutPayload)
{
	OutPayload.HitT = RayTCurrent();
	OutPayload.GPUSceneInstanceId = GetInstanceUserData();
	OutPayload.Barycentrics = Attributes.GetBarycentrics();

	//Reconstruct world position.
	OutPayload.TranslatedWorldPosition = TranslatedWorldRayOrigin() + WorldRayDirection() * RayTCurrent();

	//Reconstruct world normal.
	FTriangleBaseAttributes Tri = LoadTriangleBaseAttributes(PrimitiveIndex());

	float3 LocalPosition = (1.0f - OutPayload.Barycentrics.x - OutPayload.Barycentrics.y) * Tri.LocalPositions[0]
		+ OutPayload.Barycentrics.x * Tri.LocalPositions[1]
		+ OutPayload.Barycentrics.y * Tri.LocalPositions[2];

	float3 LocalEdges[2] = { Tri.LocalPositions[1] - Tri.LocalPositions[0], Tri.LocalPositions[2] - Tri.LocalPositions[0] };
	float3 LocalNormal = cross(LocalEdges[1], LocalEdges[0]);
	float3x3 InverseTranspose3x3 = transpose((float3x3)WorldToObject4x3());

	OutPayload.WorldNormal = normalize(mul(LocalNormal, InverseTranspose3x3));
}

void VfxTraceRay(
	RaytracingAccelerationStructure AccelerationStructure,
	uint RayFlags,
	uint InstanceInclusionMask,
	uint RayContributionToHitGroupIndex,
	uint MultiplierForGeometryContributionToHitGroupIndex,
	uint MissShaderIndex,
	RayDesc InRay,
	inout FVFXTracePayload Payload)
{
	TraceRay(AccelerationStructure, RayFlags, InstanceInclusionMask, RayContributionToHitGroupIndex, MultiplierForGeometryContributionToHitGroupIndex, MissShaderIndex, InRay, Payload);
}

#endif //#ifndef OVERRIDE_VFXTRACERAY_USH

#if COMPUTESHADER

StructuredBuffer<FRayTracingSceneMetadataRecord> RayTracingSceneMetadata;

void VfxTraceRayInline(
	RaytracingAccelerationStructure AccelerationStructure,
	uint RayFlags,
	uint InstanceInclusionMask,
	RayDesc InRay,

	inout FVFXTracePayload Payload)
{
	FTraceRayInlineContext TraceContext = CreateTraceRayInlineContext();
	FTraceRayInlineResult TraceResult = TraceRayInline(AccelerationStructure, RayFlags, InstanceInclusionMask, InRay, TraceContext);
	if (TraceResult.IsHit())
	{
		Payload.HitT = TraceResult.HitT;
		Payload.GPUSceneInstanceId = TraceResult.InstanceID;
		Payload.Barycentrics = TraceResult.Barycentrics;

#if PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_NORMALS
		Payload.WorldNormal = TraceResult.WorldGeometryNormal;
#elif PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
		FTriangleBaseAttributes Attributes = LoadInlineRayTracingTriangleAttributes(
			RayTracingSceneMetadata,
			TraceResult.InstanceContributionToHitGroupIndex,
			RAY_TRACING_NUM_SHADER_SLOTS,
			TraceResult.GeometryIndex,
			TraceResult.PrimitiveIndex
		);
		float3 LocalNormal = cross(Attributes.LocalPositions[2] - Attributes.LocalPositions[0], Attributes.LocalPositions[1] - Attributes.LocalPositions[0]);
		Payload.WorldNormal = normalize(mul(LocalNormal, (float3x3)TraceResult.WorldToObject3x4));	
#else // !PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
		Payload.WorldNormal = 0.0f;
#endif // !PLATFORM_SUPPORTS_INLINE_RAY_TRACING_TRIANGLE_ATTRIBUTES
		Payload.TranslatedWorldPosition = InRay.Origin + InRay.Direction * TraceResult.HitT;
	}
	else
	{
		Payload.SetMiss();
	}
}

#endif

#endif // UE_TRACE_RAY_USH_INCLUDED // Workarround for UE-66460
