// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RecomputeTangentsCommon.usf: Recompute Skin Tangents
=============================================================================*/

#include "Common.ush"
#include "GpuSkinCommon.ush"

// constants -----------------------------

// could be larger for better precision but we need to be conservative as we sum up many triangles weighed by the angle (potentially unbound)
#define TANGENT_RANGE 0x7fff

// chunk -----------------------------

//
uint NumVertices;
//
uint NumTriangles;

// vertex input ----------------------------

//

Buffer<SNORM float4> TangentInputBuffer;
Buffer<float2> UVsInputBuffer;
RWBuffer<TANGENT_RWBUFFER_FORMAT> TangentBufferUAV;

// in vertices
uint InputStreamStart;
uint NumTexCoords;

// index input ----------------------------

// start index in IndexBuffer[]
uint IndexBufferOffset;
// todo: test 16bit and 32 bit 
/// only needed for MainCS()
Buffer<uint> IndexBuffer;

// SkinCache -----------------------------

// input vertex data (SkinCache output)
// used as input in MainCS()
Buffer<float4> GPUTangentCacheBuffer;
Buffer<float> GPUPositionCacheBuffer;

// start offset in vertices
uint SkinCacheStart;

// intermediate ------------------------------

// intermediate tangent accumulation buffer, int se we can use atomics and to get deterministic results no matter what order
// uses as output on MainCS(), used and input and output (to clear) for ResolveCS()
RWBuffer<int> IntermediateAccumBufferUAV;
uint IntermediateAccumBufferOffset;