// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RecomputeTangentsPerTrianglePass.usf: Recompute Skin Tangents
=============================================================================*/

#include "RecomputeTangentsCommon.ush"
#include "FastMath.ush"

#if MERGE_DUPLICATED_VERTICES 
Buffer<uint> DuplicatedIndices;
Buffer<uint> DuplicatedIndicesIndices;
#endif
struct FVertexData
{
	// vertex index in the vertexbuffer
	uint VertexIndex;
	// position including skinning and morph target animation
	float3 Position;
	// 
	float2 UV;
	// 0..1, used for weighting the triangle contributions
	float Angle;
	// Original sign from the original tangent basis, as some meshes have left handed triangles in them
	float OriginalOrientation;
};

// can be optimized
// @return not neccessarily normalized
float3 FindPerpendicularTo(float3 In)
{
	float3 Other = (abs(normalize(In).z) < 0.5f)
		? float3(0,0,1)
		: float3(1,0,0);

	return cross(Other, In);
}

FVertexData GetVertex(uint VertexIndex)
{
	FVertexData Ret;

	// is filled out later
	Ret.Angle = 0;

	// no start offset as we reuse the same buffer always from 0 on
	Ret.VertexIndex = VertexIndex;

	{
		uint OutputOffset = VertexIndex + SkinCacheStart;
	
		Ret.Position.x = GPUPositionCacheBuffer[3 * OutputOffset + 0];
		Ret.Position.y = GPUPositionCacheBuffer[3 * OutputOffset + 1];
		Ret.Position.z = GPUPositionCacheBuffer[3 * OutputOffset + 2];

		float W = GPUTangentCacheBuffer[2 * OutputOffset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z].w;
		Ret.OriginalOrientation = (W <= 0) ? -1 : 1;
	}

	{
		uint InputOffset = VertexIndex + InputStreamStart;

		Ret.UV = UVsInputBuffer[InputOffset * NumTexCoords];
	}

	return Ret;
}

[numthreads(THREADGROUP_SIZEX, 1, 1)]
void MainCS(
	uint DispatchThreadId : SV_DispatchThreadID)	// DispatchThreadId = GroupId * int2(dimx,dimy) + GroupThreadId
{
	uint TriangleIndex = DispatchThreadId;

	// We need this to restrict processing of triangles to the valid range (We run a Dispatch with a coarse granularity and would process too many elements without that)
	if(TriangleIndex < NumTriangles)
	{
		// 3 triangle corners
		FVertexData Corner[3];
		{
			// 3 vertices per triangle
			uint TriangleIndexBufferStart = TriangleIndex * 3 + IndexBufferOffset;

			UNROLL for(uint CornerId = 0; CornerId < 3; ++CornerId)
			{
				uint VertexIndex = IndexBuffer[TriangleIndexBufferStart + CornerId] - InputStreamStart;

				Corner[CornerId] = GetVertex(VertexIndex);
			}
		}

		// Compute the angle in each corner for weighting
		{
			UNROLL for(uint CornerId = 0; CornerId < 3; ++CornerId)
			{
				float3 Pos = Corner[CornerId].Position;

				float3 EdgeA = Corner[(CornerId + 1) % 3].Position - Pos;
				float3 EdgeB = Corner[(CornerId + 2) % 3].Position - Pos;
			
				Corner[CornerId].Angle = AngleBetweenVectorsFast(EdgeA, EdgeB);
			}
		}

		float3 EdgeA = Corner[1].Position - Corner[0].Position;
		float3 EdgeB = Corner[2].Position - Corner[0].Position;

		float3 TriangleNormal = cross(EdgeB, EdgeA);

		// in -1..1 range
		float3 TangentZ = normalize(TriangleNormal);
		
		// to avoid having degenerated triangles causing shading artifacts on neighbors (should be compiled out if the boolean is not use)
		bool bDegenerated = false;

		bDegenerated = length2(TriangleNormal) < 0.000001f;

		float3 TangentX;
		float Orientation;
		{
			float2 UVEdgeA = Corner[1].UV - Corner[0].UV;
			float2 UVEdgeB = Corner[2].UV - Corner[0].UV;

			// as explained here: http://www.3dkingdoms.com/weekly/weekly.php?a=37
			float cp = UVEdgeA.y * UVEdgeB.x - UVEdgeA.x * UVEdgeB.y;

			float3 UnnormalizedTangentX = (EdgeA * -UVEdgeB.y + EdgeB * UVEdgeA.y) / cp;
			// normally this should be normalize(x / cp) but we only need the sign of cp so we can multiply as well, which is faster and avoids a division by 0
//			float3 UnnormalizedTangentX = (EdgeA * -UVEdgeB.y + EdgeB * UVEdgeA.y) * cp;

			// bad UV assignment
			if(abs(cp) < 0.000001f)
			{
				UnnormalizedTangentX = FindPerpendicularTo(TangentZ);
			}
			
			FLATTEN if(length2(UnnormalizedTangentX) < 0.000001f)
			{
			// EdgeA is in the direction EdgeB
				bDegenerated = true;
			}

			TangentX = normalize(UnnormalizedTangentX);

			Orientation = (cp > 0) ? 1 : -1;
			// Multiply by additional flip if the triangle had it's tangent space inverted
			Orientation *= Corner[0].OriginalOrientation;
		}

/* disabled for now (for performance) but if needed we can enable it later
		if(bDegenerated)
		{
			TangentX = 0;
			TangentZ = 0;
			Orientation = 0;
		}
*/
		{
			UNROLL for(uint CornerId = 0; CornerId < 3; ++CornerId)
			{
				// weighting by the angle results in more correct results with triangle meshes (independent of diagonal edges)
				float Weight = Corner[CornerId].Angle;

				// in -TANGENT_RANGE*PI..TANGENT_RANGE*PI  range
				int3 IntTangentZ = (int3)(TangentZ * Weight * TANGENT_RANGE);
				int3 IntTangentX = (int3)(TangentX * Weight * TANGENT_RANGE);
				int IntOrientation = (int)(Orientation * Weight * TANGENT_RANGE);

				uint VertexIndex = Corner[CornerId].VertexIndex;

				// accumulate the contributions of each triangle on the triangle vertices
				
				// no start offset as we reuse the same buffer always from 0 on
				uint Index = (IntermediateAccumBufferOffset + VertexIndex) * INTERMEDIATE_ACCUM_BUFFER_NUM_INTS;

				// TangentZ
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 0], IntTangentZ.x);
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 1], IntTangentZ.y);
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 2], IntTangentZ.z);
				// TangentX
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 3], IntTangentX.x);
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 4], IntTangentX.y);
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 5], IntTangentX.z);
				// Orientation
				InterlockedAdd(IntermediateAccumBufferUAV[Index + 6], IntOrientation);
                
#if MERGE_DUPLICATED_VERTICES 
                uint DupVertexIndicesLength = DuplicatedIndicesIndices[2 * VertexIndex];
                uint DupIndexStart = DuplicatedIndicesIndices[2 * VertexIndex + 1];
                for(uint DupIndexOffset = 0; DupIndexOffset < DupVertexIndicesLength; ++DupIndexOffset)
                {
                    uint DupVertexIndex = DuplicatedIndices[DupIndexStart + DupIndexOffset];
                    uint DupIndex = (IntermediateAccumBufferOffset + DupVertexIndex) * INTERMEDIATE_ACCUM_BUFFER_NUM_INTS;
                    // TangentZ
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 0], IntTangentZ.x);
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 1], IntTangentZ.y);
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 2], IntTangentZ.z);
                    // TangentX
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 3], IntTangentX.x);
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 4], IntTangentX.y);
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 5], IntTangentX.z);
                    // Orientation
                    InterlockedAdd(IntermediateAccumBufferUAV[DupIndex + 6], IntOrientation);
                }
#endif
			}
		}
	}
}

