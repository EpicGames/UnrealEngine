// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RecomputeTangentsPerVertexPass.usf: Recompute Skin Tangents
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "RecomputeTangentsCommon.ush"

// 0:fast and simple / 1:higher quality lighting with extreme mesh distortions
#define ORTHOGONALIZE 1

#ifndef BLEND_USING_VERTEX_COLOR
#define BLEND_USING_VERTEX_COLOR 0
#endif

#if BLEND_USING_VERTEX_COLOR
Buffer<float4> ColorInputBuffer;
#endif

int VertexColorChannel;

float4 TangentToQuaternion(float3 TangentX, float3 TangentZ)
{
	// Based off \Engine\Source\Runtime\Core\Public\Math\Quat.h FQuat(const FMatrix& M)
	// Be sure TangentX and TangentX are normalized
	float3x3 M;
	M[1] = cross(TangentZ, TangentX);
	M[2] = TangentZ;
	M[0] = cross(M[1], TangentZ.xyz);	// Recompute TangentX in case TangentX & Z aren't perfectly perpendicular to each other

	float4 Quat;
	float S;

	// Check diagonal (trace)
	const float Tr = M[0][0] + M[1][1] + M[2][2];
	if (Tr > 0.0f) 
	{
		float InvS = 1.0f / sqrt(Tr + 1.f);
		Quat.w = 0.5f * (1.f / InvS);
		S = 0.5f * InvS;

		Quat.x = (M[1][2] - M[2][1]) * S;
		Quat.y = (M[2][0] - M[0][2]) * S;
		Quat.z = (M[0][1] - M[1][0]) * S;
	} 
	else 
	{
		// diagonal is negative
		int i = 0;

		if (M[1][1] > M[0][0])
			i = 1;

		if (M[2][2] > M[i][i])
			i = 2;

		int nxt[3] = { 1, 2, 0 };
		int j = nxt[i];
		int k = nxt[j];
 
		S = M[i][i] - M[j][j] - M[k][k] + 1.0f;

		float InvS = 1.0f / sqrt(S);

		float Qt[4];
		Qt[i] = 0.5f * (1.0f / InvS);

		S = 0.5f * InvS;

		Qt[3] = (M[j][k] - M[k][j]) * S;
		Qt[j] = (M[i][j] + M[j][i]) * S;
		Qt[k] = (M[i][k] + M[k][i]) * S;

		Quat.x = Qt[0];
		Quat.y = Qt[1];
		Quat.z = Qt[2];
		Quat.w = Qt[3];
	}

	return normalize(Quat); 
}

void QuaternionToTangent(float4 Quat, out float3 TangentX, out float3 TangentZ)
{
	// Based off \Engine\Source\Runtime\Core\Public\Math\QuatRotationTranslationMatrix.h FQuatRotationTranslationMatrix(const FQuat& Q, const FVector& Origin)
	float x2 = Quat.x + Quat.x; float y2 = Quat.y + Quat.y; float z2 = Quat.z + Quat.z;
	float xx = Quat.x * x2;		float xy = Quat.x * y2;		float xz = Quat.x * z2;
	float yy = Quat.y * y2;     float yz = Quat.y * z2;		float zz = Quat.z * z2;
	float wx = Quat.w * x2;     float wy = Quat.w * y2;		float wz = Quat.w * z2;

	TangentX = float3(1.0f - (yy + zz), xy + wz, xz - wy);
	TangentZ = float3(xz + wy, yz - wx, 1.0f - (xx + yy));
}

float4 Slerp(float4 Start, float4 End, float T)
{
    float DotProduct = dot(Start, End);
    float DotAbs = abs(DotProduct);
	
    float T0, T1;
    if (DotAbs > 0.9999f)
    {
		// Start and End are very close together, do linear interpolation.
        T0 = 1.0f - T;
        T1 = T;
    }
    else
    {
        float Omega = acos(DotAbs);
        float InvSin = 1.f / sin(Omega);
        T0 = sin((1.f - T) * Omega) * InvSin;
        T1 = sin(T * Omega) * InvSin;
    }

    T1 = DotProduct < 0 ? -T1 : T1;

	return (T0 * Start) + (T1 * End);
}

float3 Slerp(float3 Start, float3 End, float T)
{
	float4 Result = Slerp(float4(Start, 0), float4(End, 0), T);
	return Result.xyz;
}

[numthreads(THREADGROUP_SIZEX, 1, 1)]
void MainCS(
	uint DispatchThreadId : SV_DispatchThreadID)	// DispatchThreadId = GroupId * int2(dimx,dimy) + GroupThreadId
{
	if(DispatchThreadId < NumVertices)
	{
		// -1..1 range, normalized
		float3 TangentZ, TangentX;
		// the sign defines the handyness of the tangent matrix
  
        // no start offset as we reuse the same buffer always from 0 on
        uint Index = (IntermediateAccumBufferOffset + DispatchThreadId) * INTERMEDIATE_ACCUM_BUFFER_NUM_INTS;
		
        // we don't have to scale down as we normalize anyway
        TangentZ = normalize(float3(IntermediateAccumBufferUAV[Index + 0], IntermediateAccumBufferUAV[Index + 1], IntermediateAccumBufferUAV[Index + 2]));
        TangentX = normalize(float3(IntermediateAccumBufferUAV[Index + 3], IntermediateAccumBufferUAV[Index + 4], IntermediateAccumBufferUAV[Index + 5]));

#if ORTHOGONALIZE
        // todo: can be optimized
        // derive the new tangent by orthonormalizing the new normal against
        // the base tangent vector (assuming these are normalized)
        TangentX = normalize(TangentX - (dot(TangentX, TangentZ) * TangentZ));
#endif

        float Orientation = IntermediateAccumBufferUAV[Index + 6];

		uint VertexIndex = DispatchThreadId;
		uint Offset = VertexIndex + SkinCacheStart;

#if BLEND_USING_VERTEX_COLOR
		float4 InputTangentX = TangentBias_SkinCache(TangentInputBuffer[2 * Offset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_X]);
		float4 InputTangentZ = TangentBias_SkinCache(TangentInputBuffer[2 * Offset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z]);
		float4 BlendFactors = ColorInputBuffer[DispatchThreadId + InputStreamStart] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
		float BlendFactor = (VertexColorChannel == 0) ? BlendFactors.r : ((VertexColorChannel == 1) ? BlendFactors.g : BlendFactors.b);

		// Linear interpolation
		TangentX = lerp(InputTangentX.xyz, TangentX, BlendFactor);
		TangentZ = lerp(InputTangentZ.xyz, TangentZ, BlendFactor);

		// No need to realign TangentX and TangentZ to be perfect perpendicular after blending as vertex factory will do so later on
		TangentX = normalize(TangentX);
		TangentZ = normalize(TangentZ);
		Orientation = BlendFactor < 0.5f ? InputTangentZ.w : Orientation;
#endif

		TangentBufferUAV[2 * Offset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_X] = TangentUnbias_SkinCache(float4(TangentX, 0.0f));
		TangentBufferUAV[2 * Offset + GPUSKIN_RWBUFFER_OFFSET_TANGENT_Z] = TangentUnbias_SkinCache(float4(TangentZ, Orientation));
  
        // clear input UAV to 0 again to avoid another Dispatch() call
        UNROLL for(int i = 0; i < INTERMEDIATE_ACCUM_BUFFER_NUM_INTS; ++i)
        {
            IntermediateAccumBufferUAV[Index + i] = 0;
        }
	}
}
