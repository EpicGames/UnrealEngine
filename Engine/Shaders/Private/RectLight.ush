// Copyright Epic Games, Inc. All Rights Reserved.
//

#pragma once

#include "CapsuleLight.ush"
#include "MonteCarlo.ush"
#include "LightData.ush"

#ifndef USE_SOURCE_TEXTURE
#define USE_SOURCE_TEXTURE (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define GGXLTCMatSampler	View.SharedBilinearClampedSampler
#define GGXLTCAmpSampler	View.SharedBilinearClampedSampler
#else
#define GGXLTCMatSampler	View.GGXLTCMatSampler
#define GGXLTCAmpSampler	View.GGXLTCAmpSampler
#endif

struct FRect
{
	float3		Origin;
	float3x3	Axis;
	float2		Extent;
	float2		FullExtent;
	float2		Offset;
};

float3 SampleRectTexture(FRectTexture RectTexture, float2 RectUV, float Level, bool bIsReference = false)
{
#if USE_SOURCE_TEXTURE
	const bool bIsValid = RectTexture.AtlasMaxLevel < MAX_RECT_ATLAS_MIP;
	const float2 RectTextureSize = RectTexture.AtlasUVScale * View.RectLightAtlasSizeAndInvSize.xy;
	Level += log2(min(RectTextureSize.x, RectTextureSize.y)) - 2.f;
	Level  = min(Level, RectTexture.AtlasMaxLevel);

	RectUV = saturate(RectUV) * RectTexture.AtlasUVScale + RectTexture.AtlasUVOffset;

	// Compute rect border to prevent leaking during tri-linear filtering.
	// Border are aligned on the coarsest MIP value
	const uint2 MippedResoluton = uint2(View.RectLightAtlasSizeAndInvSize.xy) >> uint(ceil(Level));
	const float2 UVBorder = 0.5f / float2(MippedResoluton);
	const float2 MinRectUV =  UVBorder + RectTexture.AtlasUVOffset;
	const float2 MaxRectUV = -UVBorder + RectTexture.AtlasUVOffset + RectTexture.AtlasUVScale;
	RectUV = clamp(RectUV, MinRectUV, MaxRectUV);

	return bIsValid ? View.RectLightAtlasTexture.SampleLevel(View.SharedTrilinearClampedSampler, RectUV, bIsReference ? 0 : Level).rgb : 1.f;
#else
	return 1;
#endif
}

// This threshold is here to avoid float explosion / NaNs at the border of rect light in sharp reflections, as well as rect light ghosts in other part of the image.
// All that can happen for relatively long and thin rect lights. The trade off is that the rect light edge can then look softer than it should be in smooht reflections for elongated rect lights.
// We can change the threshold but there will always be a size that will break the look. The only solution is to cut long rect light sources into smaller ones.
#define RECT_LIGHT_SAFE_THRESHOLD 1.0e-4f

// Optimized Lambert
float3 RectIrradianceLambert( float3 N, FRect Rect, out float BaseIrradiance, out float NoL )
{
#if 0
	float3 L0 = normalize( Rect.Origin - Rect.Axis[0] * Rect.Extent.x - Rect.Axis[1] * Rect.Extent.y );	//  8 mad, 4 mul, 1 rsqrt
	float3 L1 = normalize( Rect.Origin + Rect.Axis[0] * Rect.Extent.x - Rect.Axis[1] * Rect.Extent.y );	//  8 mad, 4 mul, 1 rsqrt
	float3 L2 = normalize( Rect.Origin + Rect.Axis[0] * Rect.Extent.x + Rect.Axis[1] * Rect.Extent.y );	//  8 mad, 4 mul, 1 rsqrt
	float3 L3 = normalize( Rect.Origin - Rect.Axis[0] * Rect.Extent.x + Rect.Axis[1] * Rect.Extent.y );	//  8 mad, 4 mul, 1 rsqrt
	// 48 alu, 4 rsqrt
#else
	float3 LocalPosition;
	LocalPosition.x = dot( Rect.Axis[0], Rect.Origin );		// 1 mul, 2 mad
	LocalPosition.y = dot( Rect.Axis[1], Rect.Origin );		// 1 mul, 2 mad
	LocalPosition.z = dot( Rect.Axis[2], Rect.Origin );		// 1 mul, 2 mad
	// 9 alu

	float x0 = LocalPosition.x - Rect.Extent.x;
	float x1 = LocalPosition.x + Rect.Extent.x;
	float y0 = LocalPosition.y - Rect.Extent.y;
	float y1 = LocalPosition.y + Rect.Extent.y;
	float z0 = LocalPosition.z;
	float z0Sqr = z0 * z0;
	// 5 alu

	float3 v0 = float3( x0, y0, z0 );
	float3 v1 = float3( x1, y0, z0 );
	float3 v2 = float3( x1, y1, z0 );
	float3 v3 = float3( x0, y1, z0 );

	float3 L0 = v0 * rsqrt( dot( v0.xy, v0.xy ) + z0Sqr );	//  2 mad, 3 mul, 1 rsqrt
	float3 L1 = v1 * rsqrt( dot( v1.xy, v1.xy ) + z0Sqr );	//  2 mad, 3 mul, 1 rsqrt
	float3 L2 = v2 * rsqrt( dot( v2.xy, v2.xy ) + z0Sqr );	//  2 mad, 3 mul, 1 rsqrt
	float3 L3 = v3 * rsqrt( dot( v3.xy, v3.xy ) + z0Sqr );	//  2 mad, 3 mul, 1 rsqrt
	// 20 alu, 4 rsqrt

	// total 34 alu, 4 rsqrt
#endif

#if 0
	float3 L;
	L  = acos( dot( L0, L1 ) ) * normalize( cross( L0, L1 ) );
	L += acos( dot( L1, L2 ) ) * normalize( cross( L1, L2 ) );
	L += acos( dot( L2, L3 ) ) * normalize( cross( L2, L3 ) );
	L += acos( dot( L3, L0 ) ) * normalize( cross( L3, L0 ) );
#else
	float c01 = dot( L0, L1 );
	float c12 = dot( L1, L2 );
	float c23 = dot( L2, L3 );
	float c30 = dot( L3, L0 );
	// 9 alu

#if 0
	float w01 = 1.5708 + (-0.879406 + 0.308609 * abs(c01) ) * abs(c01);
	float w12 = 1.5708 + (-0.879406 + 0.308609 * abs(c12) ) * abs(c12);
	float w23 = 1.5708 + (-0.879406 + 0.308609 * abs(c23) ) * abs(c23);
	float w30 = 1.5708 + (-0.879406 + 0.308609 * abs(c30) ) * abs(c30);

	w01 = c01 > 0 ? w01 : PI * rsqrt( 1 - c01 * c01 ) - w01;
	w12 = c12 > 0 ? w12 : PI * rsqrt( 1 - c12 * c12 ) - w12;
	w23 = c23 > 0 ? w23 : PI * rsqrt( 1 - c23 * c23 ) - w23;
	w30 = c30 > 0 ? w30 : PI * rsqrt( 1 - c30 * c30 ) - w30;
#else
	// normalize( cross( L0, L1 ) ) = cross( L0, L1 ) * rsqrt( 1 - dot( L0, L1 )^2 )
	// acos( x ) ~= sqrt(1 - x) * (1.5708 - 0.175 * x)

	float w01 = ( 1.5708 - 0.175 * c01 ) * rsqrt( max(c01 + 1, RECT_LIGHT_SAFE_THRESHOLD) );	// 1 mad, 1 add, 1 rsqrt
	float w12 = ( 1.5708 - 0.175 * c12 ) * rsqrt( max(c12 + 1, RECT_LIGHT_SAFE_THRESHOLD) );	// 1 mad, 1 add, 1 rsqrt
	float w23 = ( 1.5708 - 0.175 * c23 ) * rsqrt( max(c23 + 1, RECT_LIGHT_SAFE_THRESHOLD) );	// 1 mad, 1 add, 1 rsqrt
	float w30 = ( 1.5708 - 0.175 * c30 ) * rsqrt( max(c30 + 1, RECT_LIGHT_SAFE_THRESHOLD) );	// 1 mad, 1 add, 1 rsqrt
	// 8 alu, 4 rsqrt
#endif

#if 0
	float3 L;
	L  = w01 * cross( L0, L1 );	// 6 mul, 3 mad
	L += w12 * cross( L1, L2 );	// 3 mul, 6 mad
	L += w23 * cross( L2, L3 );	// 3 mul, 6 mad
	L += w30 * cross( L3, L0 );	// 3 mul, 6 mad
#else
	float3 L;
	L  = cross( L1, -w01 * L0 +  w12 * L2 );	// 6 mul, 6 mad
	L += cross( L3,  w30 * L0 + -w23 * L2 );	// 3 mul, 9 mad
#endif
#endif

	// Vector irradiance
	L = L.x * Rect.Axis[0] + L.y * Rect.Axis[1] + L.z * Rect.Axis[2];	// 3 mul, 6 mad

	float LengthSqr = dot( L, L );
	float InvLength = rsqrt( LengthSqr );
	float Length = LengthSqr * InvLength;

	// Mean light direction
	L *= InvLength;

	BaseIrradiance = 0.5 * Length;

	// Solid angle of sphere		= 2*PI * ( 1 - sqrt(1 - r^2 / d^2 ) )
	// Cosine weighted integration	= PI * r^2 / d^2
	// SinAlphaSqr = r^2 / d^2;
	float SinAlphaSqr = BaseIrradiance * (1.0 / PI);

	NoL = SphereHorizonCosWrap( dot( N, L ), SinAlphaSqr );

	return L;
}

float3 SampleSourceTexture( float3 L, FRect Rect, FRectTexture RectTexture)
{
#if USE_SOURCE_TEXTURE
	// Force to point at plane
	L += Rect.Axis[2] * saturate( 0.001 - dot( Rect.Axis[2], L ) );

	// Intersect ray with plane
	float DistToPlane = dot( Rect.Axis[2], Rect.Origin ) / dot( Rect.Axis[2], L );
	float3 PointOnPlane = L * DistToPlane;

	float2 PointInRect;
	PointInRect.x = dot( Rect.Axis[0], PointOnPlane - Rect.Origin );
	PointInRect.y = dot( Rect.Axis[1], PointOnPlane - Rect.Origin );

	// Compute UV on the original rect (i.e. unoccluded rect)
    float2 RectUV = (PointInRect + Rect.Offset) / max(0.0001f, Rect.FullExtent) * float2(0.5, -0.5) + 0.5;
	
	float Level = log2( DistToPlane * rsqrt( max(0.0001f, Rect.FullExtent.x * Rect.FullExtent.y) ) );

    return SampleRectTexture(RectTexture, RectUV, Level);
#else
	return 1;
#endif
}

float IntegrateEdge( float3 L0, float3 L1 )
{
	float c01 = dot( L0, L1 );
	//float w01 = ( 1.5708 - 0.175 * c01 ) * rsqrt( c01 + 1 );	// 1 mad, 1 mul, 1 add, 1 rsqrt
	//float w01 = 1.5708 + (-0.879406 + 0.308609 * abs(c01) ) * abs(c01);

	//return acos( c01 ) * rsqrt( 1 - c01 * c01 );

#if 0
	// [ Hill et al. 2016, "Real-Time Area Lighting: a Journey from Research to Production" ]
	float w01 = ( 5.42031 + (3.12829 + 0.0902326 * abs(c01)) * abs(c01) ) /
				( 3.45068 + (4.18814 + abs(c01)) * abs(c01) );

	w01 = c01 > 0 ? w01 : PI * rsqrt( 1 - c01 * c01 ) - w01;
#else
	float w01 = ( 0.8543985 + (0.4965155 + 0.0145206 * abs(c01)) * abs(c01) ) /
				( 3.4175940 + (4.1616724 + abs(c01)) * abs(c01) );
	
	w01 = c01 > 0 ? w01 : 0.5 * rsqrt( max(1 - c01 * c01, RECT_LIGHT_SAFE_THRESHOLD)  ) - w01;
#endif

	return w01;
}

// Optimized Lambert poly irradiance
float3 PolygonIrradiance( float3 Poly[4] )
{
	float3 L0 = normalize( Poly[0] );	//  2 mad, 4 mul, 1 rsqrt
	float3 L1 = normalize( Poly[1] );	//  2 mad, 4 mul, 1 rsqrt
	float3 L2 = normalize( Poly[2] );	//  2 mad, 4 mul, 1 rsqrt
	float3 L3 = normalize( Poly[3] );	//  2 mad, 4 mul, 1 rsqrt
	// 24 alu, 4 rsqrt

#if 0
	float3 L;
	L  = acos( dot( L0, L1 ) ) * normalize( cross( L0, L1 ) );
	L += acos( dot( L1, L2 ) ) * normalize( cross( L1, L2 ) );
	L += acos( dot( L2, L3 ) ) * normalize( cross( L2, L3 ) );
	L += acos( dot( L3, L0 ) ) * normalize( cross( L3, L0 ) );
#else
	float w01 = IntegrateEdge( L0, L1 );
	float w12 = IntegrateEdge( L1, L2 );
	float w23 = IntegrateEdge( L2, L3 );
	float w30 = IntegrateEdge( L3, L0 );

#if 0
	float3 L;
	L  = w01 * cross( L0, L1 );	// 6 mul, 3 mad
	L += w12 * cross( L1, L2 );	// 3 mul, 6 mad
	L += w23 * cross( L2, L3 );	// 3 mul, 6 mad
	L += w30 * cross( L3, L0 );	// 3 mul, 6 mad
#else
	float3 L;
	L  = cross( L1, -w01 * L0 +  w12 * L2 );	// 6 mul, 6 mad
	L += cross( L3,  w30 * L0 + -w23 * L2 );	// 3 mul, 9 mad
#endif
#endif

	// Vector irradiance
	return L;
}

struct FRectLTC
{
	float3x3 LTC;
	float3x3 InvLTC;
	float3 IrradianceScale;
};

// Return LTC coefficients for GGX-based BSDF
FRectLTC GetRectLTC_GGX(float Roughness, float3 F0, float3 F90, half NoV)
{
	float2 UV = float2( Roughness, sqrt( 1 - NoV ) );
	UV = UV * (63.0 / 64.0) + (0.5 / 64.0);
   
	float4 LTCMat = View.GGXLTCMatTexture.SampleLevel( GGXLTCMatSampler, UV, 0 );
	float4 LTCAmp = View.GGXLTCAmpTexture.SampleLevel( GGXLTCAmpSampler, UV, 0 );

	float3x3 LTC = {
		float3( LTCMat.x, 0, LTCMat.z ),
		float3(        0, 1,        0 ),
		float3( LTCMat.y, 0, LTCMat.w )
	};

	float LTCDet = LTCMat.x * LTCMat.w - LTCMat.y * LTCMat.z;

	float4 InvLTCMat = LTCMat / LTCDet;
	float3x3 InvLTC = {
		float3( InvLTCMat.w, 0,-InvLTCMat.z ),
		float3(	          0, 1,           0 ),
		float3(-InvLTCMat.y, 0, InvLTCMat.x )
	};

	FRectLTC Out = (FRectLTC)0;
	Out.LTC = LTC;
	Out.InvLTC = InvLTC;
	Out.IrradianceScale = F90 * LTCAmp.y + ( LTCAmp.x - LTCAmp.y ) * F0;
	return Out;
}

FRectLTC GetRectLTC_GGX( float Roughness, float3 SpecularColor, half NoV)
{
	// F90 is derived as, anything less than 2% is physically impossible and is instead considered to be shadowing
	const float3 F0  = SpecularColor;
	const float3 F90 = saturate(50.0 * SpecularColor);

	return GetRectLTC_GGX(Roughness, F0, F90, NoV);
}

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedSheenLTCSampler	View.SharedBilinearClampedSampler
#else
	#define SharedSheenLTCSampler	View.SheenLTCSampler
#endif

// Return LTC coefficients for Sheen BSDF
FRectLTC GetRectLTC_Sheen( float Roughness, half NoV)
{	
	const float Alpha = sqrt(Roughness);
	const float SatNoV = saturate(abs(NoV) + 1e-5);
	float2 UV = float2(Alpha, SatNoV);
	UV = UV * (31.0 / 32.0) + (0.5 / 32.0);
	const float3 SheenLTC = View.SheenLTCTexture.SampleLevel(SharedSheenLTCSampler, UV, 0).xyz;

	const float aInv = SheenLTC.x;
	const float bInv = SheenLTC.y;

	float3x3 LTC = {
		float3(aInv, 0,    bInv),
		float3(0,    aInv, 0),
		float3(0,    0,    1)
	};

	float3x3 InvLTC = {
		float3(1/aInv, 0,     -bInv/aInv),    
		float3(0,      1/aInv, 0),
		float3(0,      0,      1)
	};

	FRectLTC Out = (FRectLTC)0;
	Out.LTC = LTC;
	Out.InvLTC = InvLTC;
	Out.IrradianceScale = SheenLTC.z;
	return Out;
}

// Integrate a LTC BSDF with a rect light
// [ Heitz et al. 2016, "Real-Time Polygonal-Light Shading with Linearly Transformed Cosines" ]
float3 RectApproxLTC(FRectLTC In, half3 N, float3 V, FRect Rect, FRectTexture RectTexture, inout float3 OutMeanLightWorldDirection)
{	
	// No visibile rect light due to barn door occlusion
	if (Rect.Extent.x == 0 || Rect.Extent.y == 0) return 0;

	// Rotate to tangent space
	float3 T1 = normalize( V - N * dot( N, V ) );
	float3 T2 = cross( N, T1 );
	float3x3 TangentBasis = float3x3( T1, T2, N );

	In.LTC = mul( In.LTC, TangentBasis );
	In.InvLTC = mul( transpose( TangentBasis ), In.InvLTC );

	float3 Poly[4];
	Poly[0] = mul( In.LTC, Rect.Origin - Rect.Axis[0] * Rect.Extent.x - Rect.Axis[1] * Rect.Extent.y );
	Poly[1] = mul( In.LTC, Rect.Origin + Rect.Axis[0] * Rect.Extent.x - Rect.Axis[1] * Rect.Extent.y );
	Poly[2] = mul( In.LTC, Rect.Origin + Rect.Axis[0] * Rect.Extent.x + Rect.Axis[1] * Rect.Extent.y );
	Poly[3] = mul( In.LTC, Rect.Origin - Rect.Axis[0] * Rect.Extent.x + Rect.Axis[1] * Rect.Extent.y );

	// Vector irradiance
	float3 L = PolygonIrradiance( Poly );

	#if 0
	// Early out negative irradiance vector, causing 'ghost' rect light silhouette at low roughness
	OutMeanLightWorldDirection = N;
	if (L.z <= 0)
	{
		return 0;
	}
	#endif

	float LengthSqr = dot( L, L );
	float InvLength = rsqrt( LengthSqr );
	float Length = LengthSqr * InvLength;

	// Mean light direction
	L *= InvLength;

	// Solid angle of sphere		= 2*PI * ( 1 - sqrt(1 - r^2 / d^2 ) )
	// Cosine weighted integration	= PI * r^2 / d^2
	// SinAlphaSqr = r^2 / d^2;
	float SinAlphaSqr = Length;

	float NoL = SphereHorizonCosWrap( L.z, SinAlphaSqr );
	float Irradiance = SinAlphaSqr * NoL;

	// Kill negative and NaN
	Irradiance = -min(-Irradiance, 0.0);

#if 0
	float NoL;
	float Falloff;
	RectIrradianceLambert( N, Rect, Falloff, NoL );

	float a2 = Pow4( Roughness );
	Irradiance = Irradiance * ( 1 - a2*a2 ) + a2 * Falloff * NoL;
#endif
	
	// Transform to world space
	L = mul( In.InvLTC, L );
	OutMeanLightWorldDirection = L;

	float3 LightColor = SampleSourceTexture( L, Rect, RectTexture );
	
	return LightColor * Irradiance * In.IrradianceScale;
}

// Integrated a GGX-based BSDF with a rect light using LTC
float3 RectGGXApproxLTC( float Roughness, float3 SpecularColor, half3 N, float3 V, FRect Rect, FRectTexture RectTexture, inout float3 OutMeanLightWorldDirection)
{
	// No visibile rect light due to barn door occlusion
	if (Rect.Extent.x == 0 || Rect.Extent.y == 0) return 0;

	const float NoV = saturate( abs( dot(N, V) ) + 1e-5 );

	const FRectLTC LTC = GetRectLTC_GGX(Roughness, SpecularColor, NoV);
	return RectApproxLTC(LTC, N, V, Rect, RectTexture, OutMeanLightWorldDirection);
}

float3 RectGGXApproxLTC(float Roughness, float3 SpecularColor, half3 N, float3 V, FRect Rect, FRectTexture RectTexture)
{
	float3 MeanLightWorldDirection = 0.0f;
	return RectGGXApproxLTC(Roughness, SpecularColor, N, V, Rect, RectTexture, MeanLightWorldDirection);
}

float3 RectGGXApproxLTC(float Roughness, float3 F0, float3 F90, half3 N, float3 V, FRect Rect, FRectTexture RectTexture, inout float3 OutMeanLightWorldDirection)
{
	// No visibile rect light due to barn door occlusion
	if (Rect.Extent.x == 0 || Rect.Extent.y == 0) return 0;

	const float NoV = saturate(abs(dot(N, V)) + 1e-5);

	const FRectLTC LTC = GetRectLTC_GGX(Roughness, F0, F90, NoV);
	return RectApproxLTC(LTC, N, V, Rect, RectTexture, OutMeanLightWorldDirection);
}

float3 RectGGXApproxLTC(float Roughness, float3 F0, float3 F90, half3 N, float3 V, FRect Rect, FRectTexture RectTexture)
{
	float3 MeanLightWorldDirection = 0.0f;
	return RectGGXApproxLTC(Roughness, F0, F90, N, V, Rect, RectTexture, MeanLightWorldDirection);
}

// Integrated a Sheen-based BSDF with a rect light using LTC
float3 RectSheenApproxLTC( float Roughness, half3 N, float3 V, FRect Rect, FRectTexture RectTexture, inout float DirectionalAlbedo)
{
	// No visibile rect light due to barn door occlusion
	if (Rect.Extent.x == 0 || Rect.Extent.y == 0) return 0;

	const float NoV = saturate( abs( dot(N, V) ) + 1e-5 );

	const FRectLTC LTC = GetRectLTC_Sheen(Roughness, NoV);
	DirectionalAlbedo = LTC.IrradianceScale.x;

	const float Scale = 2 * PI; // Factor to match the path-tracer
	float3 MeanLightWorldDirection = 0.0f;
	return RectApproxLTC(LTC, N, V, Rect, RectTexture, MeanLightWorldDirection) * Scale;
}

// Rectangle projected to a sphere
// [Urena et al. 2013, "An Area-Preserving Parametrization for Spherical Rectangles"]
struct FSphericalRect
{
	float3x3	Axis;

	float		x0;
	float		x1;
	float		y0;
	float		y1;
	float		z0;

	float		b0;
	float		b1;
	float		k;
	float		SolidAngle;
};

// The routine below needs more accuracy near 1.0 to accurately capture distant rectlights
// The typical approach (for example see: asinFast) looses too much accuracy to cancelation near the endpoints
float SphericalRectAsin(float x)
{
	const float HalfPI = PI / 2;
	// Argument reduction to [0,0.5]
	// using: asin(x) == pi/2-2*asin(sqrt(0.5-0.5*x))
	float a = saturate(abs(x));
	bool inner = a < 0.5f;
	float a2 = inner ? a * a : 0.5 - 0.5 * a;
	a = inner ? a : sqrt(a2);

	// Fit using Mathematica -- max error for asin is 10^-4.87
	float r = 0.100323f;
	r = mad(r, a2, 0.163288f);
	r = mad(r, a2, 1.00011f) * a;
	r = inner ? r : HalfPI - 2 * r;

	return asfloat(asuint(r) ^ (asuint(x) & 0x80000000u)); // propagate sign bit
}

// RectZ must face origin
FSphericalRect BuildSphericalRect( FRect Rect )
{
	FSphericalRect SphericalRect;

	SphericalRect.Axis = Rect.Axis;

	float3 LocalPosition = mul(Rect.Axis, Rect.Origin);

	SphericalRect.x0 = LocalPosition.x - Rect.Extent.x;
	SphericalRect.x1 = LocalPosition.x + Rect.Extent.x;
	SphericalRect.y0 = LocalPosition.y - Rect.Extent.y;
	SphericalRect.y1 = LocalPosition.y + Rect.Extent.y;
	SphericalRect.z0 = -abs( LocalPosition.z );

	SphericalRect.Axis[2] *= LocalPosition.z > 0 ? -1 : 1;

	// Original paper creates all vertices, then all normals via cross products and finally computes angles via dot products.
	// However since all the points lay on an axis-aligned plane, the math can be simplified by recognizing that most terms become
	// zero, leaving only the product of the z components of the normals.
	float z0sq = LocalPosition.z * LocalPosition.z;
	float n0z = -SphericalRect.y0 * rsqrt(z0sq + SphericalRect.y0 * SphericalRect.y0);
	float n1z =  SphericalRect.x1 * rsqrt(z0sq + SphericalRect.x1 * SphericalRect.x1);
	float n2z =  SphericalRect.y1 * rsqrt(z0sq + SphericalRect.y1 * SphericalRect.y1);
	float n3z = -SphericalRect.x0 * rsqrt(z0sq + SphericalRect.x0 * SphericalRect.x0);

	// Original paper uses acos(-dot(...)). Here we use the identify acos(-x) == pi/2+asin(x) to simplify
	// the math.
	float G0G1 = SphericalRectAsin(n0z * n1z) + SphericalRectAsin(n1z * n2z);
	float G2G3 = SphericalRectAsin(n2z * n3z) + SphericalRectAsin(n3z * n0z);

	SphericalRect.b0 = n0z;
	SphericalRect.b1 = n2z;
	SphericalRect.k = G2G3;
	SphericalRect.SolidAngle = G0G1 + SphericalRect.k;

	return SphericalRect;
}

struct FSphericalRectSample {
	float3 Direction;
	float  Distance;
	float2 UV;
	float  InvPdf;
};

#define SPHERICAL_RECT_MIN_SOLIDANGLE 1e-3

float GetSphericalRectInversePdf(float3 Direction, float DistanceSquared, FSphericalRect Rect)
{
	if (Rect.SolidAngle > SPHERICAL_RECT_MIN_SOLIDANGLE)
	{
		// Light is big enough to use solid angle sampling
		return Rect.SolidAngle;
	}
	else
	{
		// Light is very small, area sampling is sufficient
		float Area = (Rect.y1 - Rect.y0) * (Rect.x1 - Rect.x0);
		float NoL = abs(dot(Direction, Rect.Axis[2]));
		return Area * NoL / DistanceSquared;
	}
}

FSphericalRectSample UniformSampleSphericalRect(float2 E, FSphericalRect Rect)
{
	float xu, yv;
	if (Rect.SolidAngle > SPHERICAL_RECT_MIN_SOLIDANGLE)
	{
		// Some signs are flipped from the original paper on account of the different calculation of SolidAngle and k
		float au = E.x * Rect.SolidAngle - Rect.k;
		float fu = (cos(au) * Rect.b0 + Rect.b1) / sin(au);
		float cu = rsqrt(fu * fu + Rect.b0 * Rect.b0) * (fu > 0 ? 1 : -1);
		cu = clamp(cu, -1, 1);				// avoid NaNs

		xu = -(cu * Rect.z0) * rsqrt(1 - cu * cu);
		xu = clamp(xu, Rect.x0, Rect.x1);	// avoid Infs

		float d2 = xu * xu + Rect.z0 * Rect.z0;
		float h0 = Rect.y0 * rsqrt(d2 + Rect.y0 * Rect.y0);
		float h1 = Rect.y1 * rsqrt(d2 + Rect.y1 * Rect.y1);
		float hv = h0 + E.y * (h1 - h0);
		float rv = 1.0 - hv * hv;
		yv = (rv > 0) ? (hv * d2 * rsqrt(rv * d2)) : Rect.y1;
	}
	else
	{
		// fallback to area sampling when solid angle is tiny to avoid numerical issues
		xu = lerp(Rect.x0, Rect.x1, E.x);
		yv = lerp(Rect.y0, Rect.y1, E.y);
	}
	FSphericalRectSample Result;
	Result.Direction = mul(float3(xu, yv, Rect.z0), Rect.Axis);
	Result.UV = float2(xu - Rect.x0, yv - Rect.y0) / float2(Rect.x1 - Rect.x0, Rect.y1 - Rect.y0);

	float DistanceSquared = xu * xu + yv * yv + Rect.z0 * Rect.z0;
	float InvDistance = rsqrt(DistanceSquared);

	Result.Distance = DistanceSquared * InvDistance;
	Result.Direction *= InvDistance;

	Result.InvPdf = GetSphericalRectInversePdf(Result.Direction, DistanceSquared, Rect);
	return Result;
}

FRect GetRect(
	float3 ToLight, 
	float3 LightDataDirection, 
	float3 LightDataTangent, 
	float LightDataSourceRadius, 
	float LightDataSourceLength, 
	float LightDataRectLightBarnCosAngle, 
	float LightDataRectLightBarnLength,
	bool bComputeVisibleRect)
{
	// Is blocked by barn doors
	FRect Rect;
	Rect.Origin = ToLight;
	Rect.Axis[1] = LightDataTangent;
	Rect.Axis[2] = LightDataDirection;
	Rect.Axis[0] = cross( Rect.Axis[1], Rect.Axis[2] );
	Rect.Extent = float2(LightDataSourceRadius, LightDataSourceLength);
	Rect.FullExtent = Rect.Extent;
	Rect.Offset = 0;

	// Compute the visible rectangle from the current shading point. 
	// The new rectangle will have reduced width/height, and a shifted origin
	//
	// Common setup for occlusion computation
	// Notes: Barn angle & length are identical for all sides
	//					D_B
	//					<-->
	//							D_S
	//						<--------------->     O         +X
	//					   -------------.--------------->------       ^
	//					  /        .          |                \      |
	//					 /   .                |					\     |  BarnDepth
	//				   ./                     |					 \    v
	//		     .      C                     v +Z
	//		.  
	// .
	// S
	//
	// Only compute the occluded rect if the barn door has an angle less 
	// than 88 degrees
	if (bComputeVisibleRect && LightDataRectLightBarnCosAngle > 0.035f)
	{
		const float3 LightdPdv = -Rect.Axis[1];
		const float3 LightdPdu = -Rect.Axis[0];
		const float2 LightExtent = float2(LightDataSourceRadius, LightDataSourceLength);
		const float BarnLength = LightDataRectLightBarnLength;
	
		// Project shading point S into light space
		float3 S_Light = mul(Rect.Axis, ToLight);

		// Compute barn door projection (D_B). Clamp the projection to the shading point if it is closer 
		// to the light than the actual barn door
		// Theta is the angle between the Z axis and the barn door
		const float CosTheta = LightDataRectLightBarnCosAngle;
		const float SinTheta = sqrt(1 - CosTheta * CosTheta);
		const float BarnDepth = min(S_Light.z, CosTheta * BarnLength);
		const float S_ratio = BarnDepth / max(0.0001f, CosTheta * BarnLength);
		const float D_B = SinTheta * BarnLength * S_ratio;
		
		// Clamp shading point onto the closest edge, if it is inside the rect light
		const float2 SignS = sign(S_Light.xy);
		S_Light.xy = SignS * max(abs(S_Light.xy), LightExtent + D_B.xx);
		
		// Compute the closest rect lignt corner, offset by the barn door size
		const float3 C = float3(SignS * (LightExtent + D_B.xx), BarnDepth);
			
		// Compute projected distance (D_S) of barn door onto the rect light
		// Eta is the angle between the Z axis and the direction vector (S-C)
		const float3 SProj = S_Light - C;
		const float CosEta = max(SProj.z, 0.001f);
		const float2 SinEta = abs(SProj.xy);
		const float2 TanEta = abs(SProj.xy) / CosEta;
		const float2 D_S = BarnDepth * TanEta;

		// Equivalent to (e.g., X axis):
		//  if (SignS.x < 0) MinMaxX.x += D_S - D_B;
		//  if (SignS.x > 0) MinMaxX.y -= D_S - D_B;
		const float2 MinXY = clamp(-LightExtent + (D_S - D_B.xx) * max(0, -SignS), -LightExtent, LightExtent);
		const float2 MaxXY = clamp( LightExtent - (D_S - D_B.xx) * max(0,  SignS), -LightExtent, LightExtent);
		const float2 RectOffset = 0.5f * (MinXY + MaxXY);

		Rect.Extent = 0.5f * (MaxXY - MinXY);
		Rect.Origin = Rect.Origin + LightdPdu * RectOffset.x + LightdPdv * RectOffset.y;
		Rect.Offset = -RectOffset;
		Rect.FullExtent = LightExtent;
	}

	return Rect;
}

// Get the Rect of the light with respect to the translated world position.
FRect GetRect(FLightShaderParameters In, float3 TranslatedWorldPosition)
{
	return GetRect(In.TranslatedWorldPosition - TranslatedWorldPosition,
		In.Direction,
		In.Tangent,
		In.SourceRadius,
		In.SourceLength,
		In.RectLightBarnCosAngle,
		In.RectLightBarnLength,
		true);
}

bool IsRectVisible(FRect Rect)
{
	// No-visible rect light due to barn door occlusion
	return Rect.Extent.x != 0 && Rect.Extent.y != 0;
}