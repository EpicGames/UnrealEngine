// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "ShaderPrint.ush"
#include "ColorMap.ush"

////////////////////////////////////////////////////////////////////////////////////////////////
// Clear texture
#if SHADER_CLEAR_TEXTURE

int2 Resolution;

RWTexture2D<float4> OutputTexture;

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelCoord = DispatchThreadId.xy;
	if (all(PixelCoord < Resolution))
	{
		float4 Random = 0; 
		Random.x = InterleavedGradientNoise(PixelCoord + 0.5f, 0);
		Random.y = InterleavedGradientNoise(PixelCoord + 0.5f, 1);
		Random.z = InterleavedGradientNoise(PixelCoord + 0.5f, 2);
		Random.w = 1;

		Random = float4(1, 0, 1, 1);
		OutputTexture[PixelCoord] = Random;
	}
}

#endif // SHADER_CLEAR_TEXTURE

////////////////////////////////////////////////////////////////////////////////////////////////
// Add texture
#if SHADER_RECT_VS

Buffer<uint4> SlotBuffer;
Buffer<float4> ScaleOffsetBuffer;
uint SlotBufferOffset;
int2 AtlasResolution;
int  bHasScaleOffset;

void MainVS(
	uint InVertexId : SV_VertexID,
	uint InInstanceId : SV_InstanceID,
	out float4 OutPosition : SV_Position,
	out float2 OutCoord : RECT_COORD,
	out float2 OutUV : RECT_UV,
	out uint OutId : RECT_ID)
{
	const uint4 Rect = SlotBuffer.Load(SlotBufferOffset + InInstanceId);
	const int2 RectOffset = Rect.xy;
	const int2 RectResolution = Rect.zw;
	
	// Only used for source texture, when copying source texture data into the atlas
	float4 ScaleOffset = float4(1.0f, 1.0f, 0.0f, 0.0f);
	if (bHasScaleOffset != 0)
	{
		ScaleOffset = ScaleOffsetBuffer.Load(SlotBufferOffset + InInstanceId);;
	}

	OutId = InInstanceId;

	// UV & Coords relative to the rect
	float2 SrcUV = 0;
	SrcUV.x = InVertexId == 1 || InVertexId == 2 || InVertexId == 4 ? 1.f : 0.f;
	SrcUV.y = InVertexId == 2 || InVertexId == 4 || InVertexId == 5 ? 1.f : 0.f;
	OutUV = SrcUV * ScaleOffset.xy + ScaleOffset.zw;
	OutCoord = SrcUV * RectResolution;

	// UV relative to the atlas
	const float2 UVOffset = RectOffset / float2(AtlasResolution);
	const float2 UVScale  = RectResolution / float2(AtlasResolution);
	const float2 DstUV = SrcUV * UVScale + UVOffset;
	OutPosition = float4(DstUV * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);
}
#endif // SHADER_RECT_VS

////////////////////////////////////////////////////////////////////////////////////////////////
// Add texture

#define PERMUTATION_WAVE_MIP (PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS)

#if PERMUTATION_WAVE_MIP
void FillMip(RWTexture2D<float4> OutAtlasMip, uint2 InCoord, float4 InColor, bool bWrite)
{
	float3 MipColorH = InColor.xyz + QuadReadAcrossX(InColor).xyz;
	float3 MipColor1 = MipColorH + QuadReadAcrossY(MipColorH).xyz;
	if (all((InCoord & 1) == 0) && bWrite)
	{
		const uint2 Mip1Coord = uint2(InCoord)>>1;
		OutAtlasMip[Mip1Coord] = float4(MipColor1 * 0.25, 0);
	} 
}
#endif

#if SHADER_ADD_TEXTURE

int InTextureMIPBias;
Texture2D<float4> InTexture0;
Texture2D<float4> InTexture1;
Texture2D<float4> InTexture2;
Texture2D<float4> InTexture3;
Texture2D<float4> InTexture4;
Texture2D<float4> InTexture5;
Texture2D<float4> InTexture6;
Texture2D<float4> InTexture7;

SamplerState InSampler0;
SamplerState InSampler1;
SamplerState InSampler2;
SamplerState InSampler3;
SamplerState InSampler4;
SamplerState InSampler5;
SamplerState InSampler6;
SamplerState InSampler7;

#if PERMUTATION_WAVE_MIP
RWTexture2D<float4> OutAtlasTextureMip1;
#endif

void MainPS(
	in float4 InPosition : SV_Position,
	in float2 InCoord : RECT_COORD,
	in float2 InUV : RECT_UV,
	in uint InId : RECT_ID,
	out float4 OutTexture : SV_Target0)
{
	float4 Color = 0;
	switch (InId)
	{
	case 0: Color = InTexture0.SampleLevel(InSampler0, InUV, InTextureMIPBias); break;
	case 1: Color = InTexture1.SampleLevel(InSampler1, InUV, InTextureMIPBias); break;
	case 2: Color = InTexture2.SampleLevel(InSampler2, InUV, InTextureMIPBias); break;
	case 3: Color = InTexture3.SampleLevel(InSampler3, InUV, InTextureMIPBias); break;
	case 4: Color = InTexture4.SampleLevel(InSampler4, InUV, InTextureMIPBias); break;
	case 5: Color = InTexture5.SampleLevel(InSampler5, InUV, InTextureMIPBias); break;
	case 6: Color = InTexture6.SampleLevel(InSampler6, InUV, InTextureMIPBias); break;
	case 7: Color = InTexture7.SampleLevel(InSampler7, InUV, InTextureMIPBias); break;
	}

	// Ensure there is no NaN value
	Color = -min(-Color, 0);

	#if PERMUTATION_WAVE_MIP
	FillMip(OutAtlasTextureMip1, InPosition.xy, Color, true /*bWrite*/);
	#endif

	OutTexture = Color;
}

#endif // SHADER_ADD_TEXTURE

////////////////////////////////////////////////////////////////////////////////////////////////
// Copy texture
#if SHADER_COPY_TEXTURE

uint MipLevel;
Buffer<uint4> SrcSlotBuffer;

Texture2D<float4> SourceAtlasTexture;
SamplerState InSampler;

void MainPS(
	in float4 InPosition : SV_Position,
	in float2 InCoord : RECT_COORD,
	in float2 InUV : RECT_UV,
	in uint InId : RECT_ID,
	out float4 OutTexture : SV_Target0)
{
	const uint2 SrcRectOffset = SrcSlotBuffer[InId].xy;
	const uint2 SourceCoord = InCoord + SrcRectOffset;
	const float4 Color = SourceAtlasTexture.Load(uint3(SourceCoord, MipLevel));
	OutTexture = Color;
}

#endif // SHADER_COPY_TEXTURE

////////////////////////////////////////////////////////////////////////////////////////////////
// Filter texture
#if SHADER_FILTER_TEXTURE

float KernelSize;
uint FilterQuality;
uint bWriteMIP1;
int2 SrcAtlasResolution;
Buffer<uint4> SrcSlotBuffer;
Texture2D<float4> SourceAtlasTexture;
SamplerState SourceAtlasSampler;

#if PERMUTATION_WAVE_MIP
RWTexture2D<float4> OutAtlasTextureMip1;
#endif

float3 FilterTap(float Weight, float2 Coord, float2 MinCoord, float2 MaxCoord)
{
	Coord = clamp(Coord, MinCoord + 0.5, MaxCoord - 0.5);
	float2 UV = Coord / SrcAtlasResolution;

	// MIP=0 as it is an SRV narrowed around SrcMIP
	return Weight * SourceAtlasTexture.SampleLevel(SourceAtlasSampler, UV, 0).xyz;
}

void MainPS(
	in float4 InPosition : SV_Position,
	in float2 InCoord : RECT_COORD,
	in float2 InUV : RECT_UV,
	in uint InId : RECT_ID,
	out float4 OutTexture : SV_Target0)
{

	if (FilterQuality == 0)
	{
		// 2x2 box filtering
		const float2 SrcCoord = uint2(InCoord) * 2.f;
		const uint4 SrcRect = SrcSlotBuffer[InId];
		const uint2 SrcRectOffset = SrcRect.xy;
		const uint2 SrcRectResolution = SrcRect.zw;
		const uint2 SrcRectMax = SrcRectOffset + SrcRectResolution;

		const float2 Coord00 = SrcRectOffset + SrcCoord + float2(0,0);
		const float2 Coord10 = SrcRectOffset + SrcCoord + float2(1,0);
		const float2 Coord01 = SrcRectOffset + SrcCoord + float2(0,1);
		const float2 Coord11 = SrcRectOffset + SrcCoord + float2(1,1);

		const float W00 = all(Coord00 < SrcRectMax) ? 1.f : 0.f;
		const float W10 = all(Coord10 < SrcRectMax) ? 1.f : 0.f;
		const float W01 = all(Coord01 < SrcRectMax) ? 1.f : 0.f;
		const float W11 = all(Coord11 < SrcRectMax) ? 1.f : 0.f;
		const float InvW = 1.f / max(1.f, float(W00 + W10 + W01 + W11));

		const float4 Color0 = W00 > 0.0 ? SourceAtlasTexture.Load(uint3(Coord00, 0)) : 0.0;
		const float4 Color1 = W10 > 0.0 ? SourceAtlasTexture.Load(uint3(Coord10, 0)) : 0.0;
		const float4 Color2 = W01 > 0.0 ? SourceAtlasTexture.Load(uint3(Coord01, 0)) : 0.0;
		const float4 Color3 = W11 > 0.0 ? SourceAtlasTexture.Load(uint3(Coord11, 0)) : 0.0;

		const float4 MipColor = (W00*Color0 + W10*Color1 + W01*Color2 + W11*Color3) * InvW;

		#if PERMUTATION_WAVE_MIP
		FillMip(OutAtlasTextureMip1, InPosition.xy, MipColor, bWriteMIP1);
		#endif

		OutTexture = MipColor;
	}
	else
	{
		// Gaussian filtering approximation from [Jimenez 2014, "Next generation post processing in Call of Duty: Advanced Warfare"]
		// 
		// Possible improvement, but probably not worth it since the filtering is done only once, when the texture is uploaded: 
		// * Separable filter (but requires compute with border, or RT ping-pong)
		// * Recursive sparse filter with MIP prefiltering (Deriche & co)

		const uint4 SrcRect = SrcSlotBuffer[InId];
		const uint2 SrcRectOffset = SrcRect.xy;
		const uint2 SrcRectResolution = SrcRect.zw;
		float2 CenterCoord = SrcRectOffset + InCoord * 2.0; // -> InCoord is xxx.5f, so SrcCoord is at the center of the 2x2 block
		const float2 MinCoord = SrcRectOffset;
		const float2 MaxCoord = SrcRectOffset + SrcRectResolution;

		float3 MipColor = 0.0;

		// Center
		MipColor += FilterTap(0.125, CenterCoord, MinCoord, MaxCoord);

		// Center square
		MipColor += FilterTap(0.125, CenterCoord + float2( 1.0,  1.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.125, CenterCoord + float2(-1.0,  1.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.125, CenterCoord + float2(-1.0, -1.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.125, CenterCoord + float2( 1.0, -1.0), MinCoord, MaxCoord);

		// Edges
		MipColor += FilterTap(0.0625, CenterCoord + float2( 2.0,  0.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.0625, CenterCoord + float2(-2.0,  0.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.0625, CenterCoord + float2( 0.0,  2.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.0625, CenterCoord + float2( 0.0, -2.0), MinCoord, MaxCoord);

		// Corners
		MipColor += FilterTap(0.03125, CenterCoord + float2( 2.0,  2.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.03125, CenterCoord + float2(-2.0,  2.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.03125, CenterCoord + float2(-2.0, -2.0), MinCoord, MaxCoord);
		MipColor += FilterTap(0.03125, CenterCoord + float2( 2.0, -2.0), MinCoord, MaxCoord);

		OutTexture = float4(MipColor, 1.0);
	}
}

#endif // SHADER_FILTER_TEXTURE

////////////////////////////////////////////////////////////////////////////////////////////////
// Debug
#if SHADER_DEBUG

int2  AtlasResolution;
float AtlasMaxMipLevel;
float Occupancy;
uint  SlotCount;
uint  HorizonCount;
uint  FreeCount;
int2  OutputResolution;
uint  AtlasMIPIndex;
uint  AtlasSourceTextureMIPBias;
uint  AtlasFormat;

uint  ForceUpdate;
uint  ResetOnChange;
uint  ClearAtlas;

SamplerState AtlasSampler;
Texture2D<float4> AtlasTexture;
RWTexture2D<float4> OutputTexture;
Buffer<uint4> SlotBuffer;
Buffer<uint4> HorizonBuffer;
Buffer<uint4> FreeBuffer;

FFontColor GetOccupancyColor(float In)
{
	float3 Color = lerp(float3(0, 1, 0), float3(1, 0, 0), saturate(In));
	return InitFontColor(Color);
}

bool IsInSlot(float2 Coord, float2 MinRect, float2 MaxRect)
{
	return all(Coord >= MinRect) && all(Coord <= MaxRect);
}

bool IsOnBorder(float2 Coord, float2 MinRect, float2 MaxRect, float BorderSize)
{
	const bool bIsWithin = IsInSlot(Coord, MinRect, MaxRect);
	const bool bIsOnBorder = any(Coord - MinRect <= BorderSize) || any(MaxRect - Coord  <= BorderSize);
	return bIsWithin && bIsOnBorder;
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Draw stats
	if (all(DispatchThreadId == 0))
	{
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 100));

		Print(Context, TEXT("Atlas resolution : "), FontSilver);
		Print(Context, AtlasResolution, FontOrange);
		Newline(Context);

		Print(Context, TEXT("Atlas max. MIP   : "), FontSilver);
		Print(Context, AtlasMaxMipLevel, FontOrange);
		Newline(Context);

		Print(Context, TEXT("Atlas MIP Index  : "), FontSilver);
		Print(Context, AtlasMIPIndex, FontOrange);
		Newline(Context);

		const FFontColor OccupancyColor = GetOccupancyColor(Occupancy);
		Print(Context, TEXT("Atlas occupancy  : "), FontSilver);
		Print(Context, Occupancy * 100.f, OccupancyColor);
		Newline(Context);

		Print(Context, TEXT("Textures count   : "), FontSilver);
		Print(Context, SlotCount, FontOrange);
		Newline(Context);

		Print(Context, TEXT("Src. MIP bias    : "), FontSilver);
		Print(Context, AtlasSourceTextureMIPBias, FontOrange);
		Newline(Context);

		Print(Context, TEXT("Atlas format     : "), FontSilver);
		if (AtlasFormat == 0) 
		{ Print(Context, TEXT("PF_FloatR11G11B10"), FontOrange); }
		else if (AtlasFormat == 1) 
		{ Print(Context, TEXT("PF_FloatRGBA"), FontOrange); }
		else  
		{ Print(Context, TEXT("Unknown"), FontOrange); }
		Newline(Context);

		Print(Context, TEXT("Force Update     : "), FontSilver);
		PrintBool(Context, ForceUpdate);
		Newline(Context);

		Print(Context, TEXT("Reset On Change  : "), FontSilver);
		PrintBool(Context, ResetOnChange);
		Newline(Context);

		Print(Context, TEXT("Clear Atlas      : "), FontSilver);
		PrintBool(Context, ClearAtlas);
		Newline(Context);

	}

	// Draw atlas
	const uint2 Coord = DispatchThreadId.xy;
	const float2 OutputUV = float2(DispatchThreadId.xy) / float2(OutputResolution);
	const float AtlasRatio  = AtlasResolution.y / float(AtlasResolution.x);
	const float OutputRatio = OutputResolution.y / float(OutputResolution.x);

	const float ScaleFactor = 0.25f;
	const uint2 DisplayAtlasOffset     = uint2(50, 250);
	const uint2 DisplayAtlasResolution = uint2(OutputResolution.x * ScaleFactor, OutputResolution.x * ScaleFactor * AtlasResolution.y / AtlasResolution.x);
	if (all(Coord > DisplayAtlasOffset) && all(Coord < DisplayAtlasOffset + DisplayAtlasResolution))
	{
		const float2 AtlasUV	= (Coord - DisplayAtlasOffset) / float2(DisplayAtlasResolution);
		const float2 AtlasCoord = AtlasUV * AtlasResolution;
		if (all(AtlasUV < 1.0f))
		{
			// Display the atlas, but gray scale it for easing slot visualization
			const float TintScale_Valid   = 1.0f;
			const float TintScale_Invalid = 0.25f;
			const float4 AtlasColor = AtlasTexture.SampleLevel(AtlasSampler, AtlasUV, AtlasMIPIndex);

			float4 OutColor = 0.0f; 

			// Color valid slot with a green border
			const float BorderSize =  float(AtlasResolution.x) / float(DisplayAtlasResolution.x) * 2.f;
			bool bScaleDownColor = true;

			// Valid Slot
			for (uint SlotIt = 0; SlotIt < SlotCount; ++SlotIt)
			{
				const uint4 Slot = SlotBuffer[SlotIt];
				if (IsInSlot(AtlasCoord, Slot.xy, Slot.xy + Slot.zw))
				{
					// only show actual used texture samples ?
					OutColor = AtlasColor * TintScale_Valid;
					bScaleDownColor = false;
					if (IsOnBorder(AtlasCoord, Slot.xy, Slot.xy + Slot.zw, BorderSize))
					{
						OutColor = float4(0, 1, 0, 1);
						break;
					}
				}
			}

			// Horizon 
			for (uint HorizonIt = 0; HorizonIt < HorizonCount; ++HorizonIt)
			{
				const uint4 Slot = HorizonBuffer[HorizonIt];
				if (IsInSlot(AtlasCoord, Slot.xy, Slot.xy + Slot.zw))
				{
					if (IsOnBorder(AtlasCoord, Slot.xy, Slot.xy + Slot.zw, BorderSize))
					{
						OutColor = float4(ColorMapViridis(HorizonIt / float(HorizonCount - 1)), 1);
						OutColor = float4(1, 0, 0, 1);
						break;
					}
				}
			}

			// Free rects. 
			for (uint FreeIt = 0; FreeIt < FreeCount; ++FreeIt)
			{
				const uint4 Slot = FreeBuffer[FreeIt];
				if (IsInSlot(AtlasCoord, Slot.xy, Slot.xy + Slot.zw))
				{
					OutColor = AtlasColor * TintScale_Invalid;
					if (IsOnBorder(AtlasCoord, Slot.xy, Slot.xy + Slot.zw, BorderSize))
					{
						OutColor = float4(1, 1, 0, 1);
						break;
					}
				}
			}

			OutputTexture[DispatchThreadId.xy] = float4(OutColor.xyz, 1.0f);
		}
	}
	else
	{
		OutputTexture[DispatchThreadId.xy] = 0.f;
	}
}

#endif // SHADER_DEBUG
