// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredShadingCommon.ush"
#include "MonteCarlo.ush"
#include "AreaLightCommon.ush"
#include "ShadingModels.ush"
#include "RectLight.ush"

float3 ClampToRect( float3 L, FRect Rect )
{
	// Bias toward plane
	//L -= Rect.Axis[2] * saturate( 0.001 + dot( Rect.Axis[2], L ) );
	//L = normalize( L );

	// Intersect ray with plane
	float3 PointOnPlane = L * ( dot( Rect.Axis[2], Rect.Origin ) / dot( Rect.Axis[2], L ) );
	//float3 PointOnPlane = L * ( dot( Rect.Axis[2], Rect.Origin ) / -saturate( 0.001 - dot( Rect.Axis[2], L ) ) );

	float2 PointInRect;
	PointInRect.x = dot( Rect.Axis[0], PointOnPlane - Rect.Origin );
	PointInRect.y = dot( Rect.Axis[1], PointOnPlane - Rect.Origin );

	// Clamp point to rect
	PointInRect = clamp( PointInRect, -Rect.Extent, Rect.Extent );

	float3 ToRect = Rect.Origin;
	ToRect += PointInRect.x * Rect.Axis[0];
	ToRect += PointInRect.y * Rect.Axis[1];

	return normalize( ToRect );
}

bool RayHitRect( float3 L, FRect Rect )
{
	// Intersect ray with plane
	float t = dot( Rect.Axis[2], Rect.Origin ) / dot( Rect.Axis[2], L );
	float3 PointOnPlane = L * t;

	bool InExtentX = abs( dot( Rect.Axis[0], PointOnPlane - Rect.Origin ) ) <= Rect.Extent.x;
	bool InExtentY = abs( dot( Rect.Axis[1], PointOnPlane - Rect.Origin ) ) <= Rect.Extent.y;

	return t >= 0 && InExtentX && InExtentY;
}

float IntegrateLight( FRect Rect )
{
	// No visibile rect light due to barn door occlusion
	if (Rect.Extent.x == 0 || Rect.Extent.y == 0) return 0;

	float NoL;
	float Falloff;

	// Optimized Lambert
	float3 L = RectIrradianceLambert( 0, Rect, Falloff, NoL );
	
	return Falloff;
}

FAreaLight CreateAreaLight( float Roughness, half3 N, half3 V, FRect Rect, FRectTexture SourceTexture )
{
	FAreaLight AreaLight = InitAreaLight();

	AreaLight.DiffuseL = RectIrradianceLambert( N, Rect, AreaLight.Falloff, AreaLight.NoL );
	AreaLight.SpecularL = AreaLight.DiffuseL;
	AreaLight.FalloffColor = SampleSourceTexture( AreaLight.DiffuseL, Rect, SourceTexture );
	
	AreaLight.Rect = Rect;
	AreaLight.Texture = SourceTexture;

	SetIsRectLight(AreaLight, true);
	SetAreaLightDiffuseMicroReflWeight(AreaLight, 0.0);

	return AreaLight;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting IntegrateBxDF(FGBufferData GBuffer, half3 N, half3 V, FRect Rect, FShadowTerms Shadow, FRectTexture SourceTexture)
{
	// Compute distance to light plane and cull if not larger than threshold to 
	// avoid numerical issue in (nearly) coplanar configuration
	const float Distance = dot(Rect.Axis[2], Rect.Origin);

	// No-visible rect light due to barn door occlusion
	FDirectLighting Out = (FDirectLighting)0;
	if (IsRectVisible(Rect) && Distance > 0.001f)
	{
		FAreaLight AreaLight = CreateAreaLight(GBuffer.Roughness, N, V, Rect, SourceTexture);
		GBuffer.Roughness = max(GBuffer.Roughness, 0.02);
		Out = IntegrateBxDF(GBuffer, N, V, AreaLight, Shadow);
	}
	return Out;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting IntegrateBxDF( FGBufferData GBuffer, half3 N, half3 V, FRect Rect, FShadowTerms Shadow, FRectTexture SourceTexture, uint2 SVPos )
{
	FDirectLighting Lighting = (FDirectLighting)0;

	const float SurfaceArea = 4 * Rect.Extent.x * Rect.Extent.y;
	const float SurfaceColor = 2.0 / SurfaceArea;

	// Rect normal points away from point
	if( dot( Rect.Axis[2], Rect.Origin ) < 0 )
		return Lighting;
	
	// No-visible rect light due to barn door occlusion
	if (!IsRectVisible(Rect))
		return Lighting;

	FSphericalRect SphericalRect = BuildSphericalRect( Rect );
	
	const uint NumSets = 4;
	const uint NumSamples[ NumSets ] =
	{
		0,	// Cosine hemisphere
		16,	// GGX
		0,	// Light area
		16,	// Spherical rect
	};

	uint2 SobolBase = SobolPixel( SVPos );
	uint2 SobolFrame = SobolIndex( SobolBase, View.StateFrameIndexMod8, 3 );

	UNROLL
	for( uint Set = 0; Set < NumSets; Set++ )
	{
		LOOP
		for( uint i = 0; i < NumSamples[ Set ]; i++ )
		{
			uint2 Random = Rand3DPCG16( uint3( SVPos.xy, View.Random ^ Set ) ).xy;

			float2 E = float2( SobolIndex( SobolFrame, i << 3 ) ) / 0x10000;
			//float2 E = Hammersley( i, NumSamples[ Set ], Random );
			//float2 E = CorrelatedMultiJitter2D( i, NumSamples[ Set ], Random.x );
			
			float3 L, H;
			switch( Set )
			{
				case 0:
				{
					L = TangentToWorld( CosineSampleHemisphere( E ).xyz, N );
					H = normalize( V + L );
					break;
				}
				case 1:
				{
					H = TangentToWorld( ImportanceSampleGGX( E, Pow4(GBuffer.Roughness) ).xyz, N );
					L = 2 * dot( V, H ) * H - V;
					break;
				}
				case 2:
				{
					float3 ToArea = Rect.Origin;
					ToArea += (E.x * 2 - 1) * Rect.Axis[0] * Rect.Extent.x;
					ToArea += (E.y * 2 - 1) * Rect.Axis[1] * Rect.Extent.y;
					L = normalize( ToArea );
					H = normalize( V + L );
					break;
				}
				case 3:
				{
					L = UniformSampleSphericalRect( E, SphericalRect ).Direction;
					H = normalize( V + L );
					break;
				}
			}

			float NoL = saturate( dot(N, L) );
			float NoH = saturate( dot(N, H) );
			float VoH = saturate( dot(V, H) );

			//if( NoL > 0 && VoH > 0 )
			if( VoH > 0 )
			{
				// Intersect ray with plane
				float t = dot( Rect.Axis[2], Rect.Origin ) / dot( Rect.Axis[2], L );
				float3 PointOnPlane = L * t;

				float2 PointInRect;
				PointInRect.x = dot( Rect.Axis[0], PointOnPlane - Rect.Origin );
				PointInRect.y = dot( Rect.Axis[1], PointOnPlane - Rect.Origin );

				float2 RectUV = PointInRect / Rect.Extent * float2( 0.5, -0.5 ) + 0.5;
				float3 LightColor = SampleRectTexture(SourceTexture, RectUV, 0, true);

				if( Set == 0 || Set == 1 )
				{
					bool InExtentX = abs( PointInRect.x ) <= Rect.Extent.x;
					bool InExtentY = abs( PointInRect.y ) <= Rect.Extent.y;

					BRANCH
					if( t < 0 || !InExtentX || !InExtentY )
					{
						// Missed rect
						continue;
					}
				}

				float PDF[] =
				{
					NoL * (1 / PI),
					D_GGX( Pow4(GBuffer.Roughness), NoH ) * NoH / (4 * VoH),
					dot( PointOnPlane, PointOnPlane ) / ( SurfaceArea * abs( dot( L, Rect.Axis[2] ) ) ),
					1.0 / SphericalRect.SolidAngle,
				};

				// MIS power heuristic
				float InvWeight = 0;
				UNROLL for( uint j = 0; j < NumSets; j++ )
				{
					InvWeight += Square( PDF[j] * NumSamples[j] );
				}
				float Weight = rcp( InvWeight ) * PDF[Set] * NumSamples[Set];

				FDirectLighting LightingSample = EvaluateBxDF( GBuffer, N, V, L, NoL, Shadow );

				Lighting.Diffuse		+= ( LightColor * Weight ) * LightingSample.Diffuse;
				Lighting.Specular		+= ( LightColor * Weight ) * LightingSample.Specular;
				Lighting.Transmission	+= ( LightColor * Weight ) * LightingSample.Transmission;
			}
		}
	}

	return Lighting;
}