// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush"


uint SignedRightShift(uint x, const int bitshift)
{
	if (bitshift > 0)
	{
		return x << asuint(bitshift);
	}
	else if (bitshift < 0)
	{
		return x >> asuint(-bitshift);
	}
	return x;
}


// Returns the pixel pos [[0; N[[^2 in a two dimensional tile size of N=2^TileSizeLog2, to
// store at a given SharedArrayId in [[0; N^2[[, so that a following recursive 2x2 pixel
// block reduction stays entirely LDS memory banks coherent.
uint2 InitialTilePixelPositionForReduction2x2(const uint TileSizeLog2, uint SharedArrayId)
{
	uint x = 0;
	uint y = 0;

	UNROLL
	for (uint i = 0; i < TileSizeLog2; i++)
	{
		const uint DestBitId = TileSizeLog2 - 1 - i;
		const uint DestBitMask = 1u << DestBitId;
		x |= DestBitMask & SignedRightShift(SharedArrayId, int(DestBitId) - int(i * 2 + 0));
		y |= DestBitMask & SignedRightShift(SharedArrayId, int(DestBitId) - int(i * 2 + 1));
	}

	#if 0
	const uint N = 1 << TileSizeLog2;
	return uint2(SharedArrayId / N, SharedArrayId - N * (SharedArrayId / N));
	#endif

	return uint2(x, y);
}


uint2 InitialTilePixelPositionForReduction2x2(const uint TileSizeLog2, const uint ReduceCount, uint SharedArrayId)
{
	uint2 p = InitialTilePixelPositionForReduction2x2(ReduceCount, SharedArrayId);

	SharedArrayId = SharedArrayId >> (2 * ReduceCount);

	const uint RemainingSize = 1u << (TileSizeLog2 - ReduceCount);

	p.x |= ((SharedArrayId % RemainingSize) << ReduceCount);
	p.y |= ((SharedArrayId / RemainingSize) << ReduceCount);

	return p;
}
