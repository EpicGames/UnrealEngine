// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReflectionEnvironmentComposite
=============================================================================*/

#pragma once

#include "LargeWorldCoordinates.ush"
#include "LightGridCommon.ush"

float3 CompositeReflectionCapturesAndSkylightTWS(
	float CompositeAlpha, 
	float3 TranslatedWorldPosition, 
	float3 RayDirection, 
	float Roughness, 
	float IndirectIrradiance, 
	float IndirectSpecularOcclusion,
	float3 ExtraIndirectSpecular,
	uint NumCapturesAffectingTile,
	uint CaptureDataStartIndex, 
	int SingleCaptureIndex,
	bool bCompositeSkylight
#if DIFFUSE_FROM_CAPTURE
	, half3 NormalVector,
	out half4 OutDiffuseIBL
#endif
	)
{
	float Mip = ComputeReflectionCaptureMipFromRoughness(Roughness, View.ReflectionCubemapMaxMip);
	float4 ImageBasedReflections = float4(0, 0, 0, CompositeAlpha);
	float2 CompositedAverageBrightness = float2(0.0f, 1.0f);

#if DIFFUSE_FROM_CAPTURE
	OutDiffuseIBL = half4(0.0f, 0.0f, 0.0f, 1.0f);
#endif

#if REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES
	// Accumulate reflections from captures affecting this tile, applying largest captures first so that the smallest ones display on top
	LOOP
	for (uint TileCaptureIndex = 0; TileCaptureIndex < NumCapturesAffectingTile; TileCaptureIndex++) 
	{
#if DIFFUSE_FROM_CAPTURE
		BRANCH
		if (OutDiffuseIBL.a < 0.001)
		{
			OutDiffuseIBL.a = 0.0;
			break;
		}
#else
		BRANCH
		if (ImageBasedReflections.a < 0.001)
		{
			break;
		}
#endif

		uint CaptureIndex = 0;
#ifdef REFLECTION_COMPOSITE_NO_CULLING_DATA
		CaptureIndex = TileCaptureIndex;	// Go from 0 to NumCapturesAffectingTile as absolute index in capture array
#else
		CaptureIndex = GetCulledLightDataGrid(CaptureDataStartIndex + TileCaptureIndex);
#endif

		FDFVector3 CaptureWorldPosition = MakeDFVector3(GetReflectionPositionAndRadius(CaptureIndex).xyz, GetReflectionPositionLow(CaptureIndex).xyz);
		float3 CaptureTranslatedWorldPosition = DFFastAddDemote(CaptureWorldPosition, ResolvedView.PreViewTranslation);
		float CaptureRadius = GetReflectionPositionAndRadius(CaptureIndex).w;

		float4 CaptureProperties = GetReflectionCaptureProperties(CaptureIndex);
		float CaptureFade = CaptureProperties.a;

		float3 CaptureVector = TranslatedWorldPosition - CaptureTranslatedWorldPosition;
		float CaptureVectorLength = sqrt(dot(CaptureVector, CaptureVector));		
#if DIFFUSE_FROM_CAPTURE
		// Support for normal based adjustment to capture distance.  Reduces light leaking through thin walls.
		float NormalAdjust = dot(CaptureVector, NormalVector) / max(CaptureVectorLength, 0.00001) * 50.0;		// 0.5m adjustment -- make tunable?
		float NormalizedDistanceToCapture = saturate((CaptureVectorLength + NormalAdjust) / CaptureRadius);
#else
		float NormalAdjust = 0.0;
		float NormalizedDistanceToCapture = saturate(CaptureVectorLength / CaptureRadius);
#endif

		BRANCH
		if (CaptureVectorLength < CaptureRadius)
		{
			float3 ProjectedCaptureVector = RayDirection;
			float4 CaptureOffsetAndAverageBrightness = GetReflectionCaptureOffsetAndAverageBrightness(CaptureIndex);

			// Fade out based on distance to capture
			float DistanceAlpha = 0;

#if DIFFUSE_FROM_CAPTURE
			float3 ProjectedNormalVector = NormalVector;
			float IgnoredDistanceAlpha = 0;
#endif
			
			#define PROJECT_ONTO_SHAPE 1
			#if PROJECT_ONTO_SHAPE
				#if REFLECTION_COMPOSITE_HAS_BOX_CAPTURES
					#if REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES
					// Box
					BRANCH if (CaptureProperties.b > 0)
					#endif
					{
						ProjectedCaptureVector = GetLookupVectorForBoxCapture(RayDirection, TranslatedWorldPosition, float4(CaptureTranslatedWorldPosition, CaptureRadius),
																			 GetReflectionBoxTransform(CaptureIndex), GetReflectionBoxScales(CaptureIndex), CaptureOffsetAndAverageBrightness.xyz, NormalAdjust, DistanceAlpha);
#if DIFFUSE_FROM_CAPTURE
						ProjectedNormalVector = GetLookupVectorForBoxCaptureDiffuse(NormalVector, TranslatedWorldPosition, float4(CaptureTranslatedWorldPosition, CaptureRadius),
																					GetReflectionBoxTransform(CaptureIndex), GetReflectionBoxScales(CaptureIndex), CaptureOffsetAndAverageBrightness.xyz);
#endif
					}
				#endif

				#if REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES
					// Sphere
					#if REFLECTION_COMPOSITE_HAS_BOX_CAPTURES
					else
					#endif
					{
						ProjectedCaptureVector = GetLookupVectorForSphereCapture(RayDirection, TranslatedWorldPosition, float4(CaptureTranslatedWorldPosition, CaptureRadius), NormalizedDistanceToCapture, CaptureOffsetAndAverageBrightness.xyz, DistanceAlpha);
#if DIFFUSE_FROM_CAPTURE
						ProjectedNormalVector = GetLookupVectorForSphereCapture(NormalVector, TranslatedWorldPosition, float4(CaptureTranslatedWorldPosition, CaptureRadius), NormalizedDistanceToCapture, CaptureOffsetAndAverageBrightness.xyz, IgnoredDistanceAlpha);
#endif
					}
				#endif
			#else
				DistanceAlpha = 1.0;
			#endif //PROJECT_ONTO_SHAPE

			// Fade the capture in or out over time.
			DistanceAlpha *= CaptureFade;

			float CaptureArrayIndex = CaptureProperties.g;

			{
				float4 Sample = ReflectionStruct.ReflectionCubemap.SampleLevel(ReflectionStruct.ReflectionCubemapSampler, float4(ProjectedCaptureVector, CaptureArrayIndex), Mip);

#if DIFFUSE_FROM_CAPTURE
				float DiffuseMip = View.ReflectionCubemapMaxMip - 3;		// 8x8 mip used for diffuse
				half3 DiffuseSample = ReflectionStruct.ReflectionCubemap.SampleLevel(ReflectionStruct.ReflectionCubemapSampler, float4(ProjectedNormalVector, CaptureArrayIndex), DiffuseMip).rgb;

				// Smooth blend on refresh.  PositionLow.w packs the source cubemap array index (integer part) and the
				// blend alpha (fractional part); zero means no active blend.  Sample the old cubemap and lerp toward
				// the new content as the fractional alpha decays from 1 to 0 over the configured fade time.
				float SmoothBlendPacked = GetReflectionPositionLow(CaptureIndex).w;
				BRANCH
				if (SmoothBlendPacked > 0)
				{
					float SourceArrayIndex = floor(SmoothBlendPacked);
					float BlendAlpha = SmoothBlendPacked - SourceArrayIndex;
					float4 SourceSpecular = ReflectionStruct.ReflectionCubemap.SampleLevel(ReflectionStruct.ReflectionCubemapSampler, float4(ProjectedCaptureVector, SourceArrayIndex), Mip);
					half3 SourceDiffuse = ReflectionStruct.ReflectionCubemap.SampleLevel(ReflectionStruct.ReflectionCubemapSampler, float4(ProjectedNormalVector, SourceArrayIndex), DiffuseMip).rgb;
					Sample = lerp(Sample, SourceSpecular, BlendAlpha);
					DiffuseSample = lerp(DiffuseSample, SourceDiffuse, (half)BlendAlpha);
				}
#endif

				Sample.rgb *= DistanceAlpha * CaptureProperties.r * IndirectSpecularOcclusion * ImageBasedReflections.a;
				Sample.a *= DistanceAlpha;

				// Under operator (back to front)
				ImageBasedReflections.rgb += Sample.rgb;
				ImageBasedReflections.a *= 1 - Sample.a;

				float AverageBrightness = CaptureOffsetAndAverageBrightness.w;
				CompositedAverageBrightness.x += AverageBrightness * DistanceAlpha * CompositedAverageBrightness.y;
				CompositedAverageBrightness.y *= 1 - DistanceAlpha;

#if DIFFUSE_FROM_CAPTURE
				OutDiffuseIBL.rgb += DiffuseSample * (DistanceAlpha * CaptureProperties.r * OutDiffuseIBL.a);
				OutDiffuseIBL.a *= (1 - DistanceAlpha);
#endif
			}
		}
	}

#else

	float3 ProjectedCaptureVector = RayDirection;

	FDFVector3 SingleCaptureWorldPosition = MakeDFVector3(GetReflectionPositionAndRadius(SingleCaptureIndex).xyz, GetReflectionPositionLow(SingleCaptureIndex).xyz);
	float3 SingleCaptureTranslatedWorldPosition = DFFastAddDemote(SingleCaptureWorldPosition, ResolvedView.PreViewTranslation);
	float SingleCaptureRadius = GetReflectionPositionAndRadius(SingleCaptureIndex).w;

	float4 SingleCaptureOffsetAndAverageBrightness = GetReflectionCaptureOffsetAndAverageBrightness(SingleCaptureIndex);
	float SingleCaptureBrightness = GetReflectionCaptureProperties(SingleCaptureIndex).x;
	float SingleCaptureArrayIndex = GetReflectionCaptureProperties(SingleCaptureIndex).y;

	#define APPROXIMATE_CONTINUOUS_SINGLE_CAPTURE_PARALLAX 0
	#if APPROXIMATE_CONTINUOUS_SINGLE_CAPTURE_PARALLAX
		float3 CaptureVector = TranslatedWorldPosition - SingleCaptureTranslatedWorldPosition;
		float CaptureVectorLength = sqrt(dot(CaptureVector, CaptureVector));		
		float NormalizedDistanceToCapture = saturate(CaptureVectorLength / SingleCaptureRadius);

		float UnusedDistanceAlpha = 0;
		ProjectedCaptureVector = GetLookupVectorForSphereCapture(RayDirection, TranslatedWorldPosition, float4(SingleCaptureTranslatedWorldPosition, SingleCaptureRadius), NormalizedDistanceToCapture, SingleCaptureOffsetAndAverageBrightness.xyz, UnusedDistanceAlpha);
		
		float x = saturate(NormalizedDistanceToCapture);
		float DistanceAlpha = 1 - x * x * (3 - 2 * x);
		// Lerp between sphere parallax corrected and infinite based on distance to shape
		ProjectedCaptureVector = lerp(RayDirection, normalize(ProjectedCaptureVector), DistanceAlpha);
	#endif

	float4 Sample = TextureCubeArraySampleLevel(ReflectionStruct.ReflectionCubemap, ReflectionStruct.ReflectionCubemapSampler, ProjectedCaptureVector, SingleCaptureArrayIndex, Mip);

	Sample.rgb *= SingleCaptureBrightness;
	ImageBasedReflections = float4(Sample.rgb, 1 - Sample.a);

	float AverageBrightness = SingleCaptureOffsetAndAverageBrightness.w;
	CompositedAverageBrightness.x += AverageBrightness * CompositedAverageBrightness.y;
	CompositedAverageBrightness.y = 0;
#endif

	// Apply indirect lighting scale while we have only accumulated reflection captures
	ImageBasedReflections.rgb *= View.PrecomputedIndirectSpecularColorScale;
	CompositedAverageBrightness.x *= Luminance( View.PrecomputedIndirectSpecularColorScale );

#if ENABLE_SKY_LIGHT
#if DIFFUSE_FROM_CAPTURE
	float SkylightReflectionScale = OutDiffuseIBL.a;
#else
	float SkylightReflectionScale = 1.0f;
#endif

	BRANCH  
	if (SkylightReflectionScale > 0.0f && ReflectionStruct.SkyLightParameters.y > 0 && bCompositeSkylight)
	{
		float SkyAverageBrightness = 1.0f;

		#if REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND
			float3 SkyLighting = GetSkyLightReflectionSupportingBlend(RayDirection, Roughness, SkyAverageBrightness);
		#else
			float3 SkyLighting = GetSkyLightReflection(RayDirection, Roughness, SkyAverageBrightness);
		#endif

		// Normalize for static skylight types which mix with lightmaps, material ambient occlusion as well as diffuse/specular occlusion.
		bool bNormalize = ReflectionStruct.SkyLightParameters.z < 1 && ALLOW_STATIC_LIGHTING;

		FLATTEN
		if (bNormalize)
		{
			ImageBasedReflections.rgb += ImageBasedReflections.a * SkyLighting * (IndirectSpecularOcclusion * SkylightReflectionScale);
			CompositedAverageBrightness.x += SkyAverageBrightness * CompositedAverageBrightness.y;
		}
		else
		{
			ExtraIndirectSpecular += SkyLighting * (IndirectSpecularOcclusion * SkylightReflectionScale);
		}
	}
#endif

#if ALLOW_STATIC_LIGHTING
	ImageBasedReflections.rgb *= ComputeMixingWeight(IndirectIrradiance, CompositedAverageBrightness.x, Roughness);
#endif

	ImageBasedReflections.rgb += ImageBasedReflections.a * ExtraIndirectSpecular;

	return ImageBasedReflections.rgb;
}
