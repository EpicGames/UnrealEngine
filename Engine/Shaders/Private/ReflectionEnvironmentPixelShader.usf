// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReflectionEnvironmentComputeShaders - functionality to apply local cubemaps.
=============================================================================*/
  
#if SUBSTRATE_ENABLED
#define SUBSTRATE_INLINE_SHADING 0
  #if SUBSTRATE_TILETYPE == 0
	#define SUBSTRATE_FASTPATH 1
  #elif SUBSTRATE_TILETYPE == 1
	#define SUBSTRATE_SINGLEPATH 1
  #elif SUBSTRATE_TILETYPE == 2
	// COMPLEX PATH
  #elif SUBSTRATE_TILETYPE == 3
	// COMPLEX PATH
	#define SUBSTRATE_COMPLEXSPECIALPATH 1
  #else
	#error Substrate tile type non-implemented
  #endif
#endif

//#define SceneTexturesStruct SceneTextures 

#if SUBSTRATE_GBUFFER_FORMAT==0
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 0	// Load from a specified BSDF, not from MAterialBuffer.
#define SUBSTRATE_INLINE_SHADING 1				// Inline shading for normal setup if needed.
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 1		// Need for SSS dither.
#include "Substrate/SubstrateRead.ush"
#else 
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 1
#endif

#include "Common.ush" 
#include "DeferredShadingCommon.ush"  
#include "BRDF.ush"
#include "ReflectionEnvironmentShared.ush"
#include "SkyLightingShared.ush"
#include "SkyLightingDiffuseShared.ush"
#include "DistanceFieldAOShared.ush"
#include "ShadingModels.ush"
#include "LightGridCommon.ush"
#include "SceneTextureParameters.ush"
#include "ClearCoatCommon.ush"
#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES 1
#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 1
#define USE_SUBSTRATE_ENV_LIGHTING_COMMON 1
#include "ReflectionEnvironmentComposite.ush"
#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#include "/Engine/Private/Substrate/SubstrateLightingCommon.ush"
#include "Lumen/LumenReflectionsCombine.ush"
#include "PostProcessSubsurfaceDiffuse.ush"

#define REFLECTION_SOURCE_SSR				0
#define REFLECTION_SOURCE_TILED_SSR			1
#define REFLECTION_SOURCE_LUMEN_STANDALONE	2

#ifndef REFLECTION_SOURCE_TYPE
#define REFLECTION_SOURCE_TYPE REFLECTION_SOURCE_SSR
#endif

#if REFLECTION_SOURCE_TYPE != REFLECTION_SOURCE_LUMEN_STANDALONE
#include "ScreenSpaceReflectionTileCommons.ush"
#endif

#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE
Texture2DArray LumenReflectionTexture;
SamplerState LumenReflectionTextureSampler;
#else
Texture2D ReflectionTexture;
SamplerState ReflectionTextureSampler;
#endif

#if ENABLE_DYNAMIC_SKY_LIGHT
#include "VolumetricCloudCommon.ush"
Texture2D<float3> CloudSkyAOTexture;
SamplerState CloudSkyAOSampler;
float4x4 CloudSkyAOWorldToLightClipMatrix;
float CloudSkyAOFarDepthKm;
int CloudSkyAOEnabled;
#endif

Texture2D AmbientOcclusionTexture;
SamplerState AmbientOcclusionSampler;



#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_TILED_SSR
uint2 SSRTiledViewRes;
StructuredBuffer<uint> SSRTileMaskBuffer;
#endif

float3 GatherRadiance(float CompositeAlpha, float3 TranslatedWorldPosition, float3 RayDirection, float Roughness, float3 BentNormal, float IndirectIrradiance, uint ShadingModelID, uint NumCulledReflectionCaptures, uint CaptureDataStartIndex)
{
	// Indirect occlusion from DFAO, which should be applied to reflection captures and skylight specular, but not SSR
	float IndirectSpecularOcclusion = 1.0f;
	float3 ExtraIndirectSpecular = 0;

#if SUPPORT_DFAO_INDIRECT_OCCLUSION
	float IndirectDiffuseOcclusion;
	GetDistanceFieldAOSpecularOcclusion(BentNormal, RayDirection, Roughness, ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE, IndirectSpecularOcclusion, IndirectDiffuseOcclusion, ExtraIndirectSpecular);
	// Apply DFAO to IndirectIrradiance before mixing with indirect specular
	IndirectIrradiance *= IndirectDiffuseOcclusion;
#endif

	const bool bCompositeSkylight = true;
	return CompositeReflectionCapturesAndSkylightTWS(
		CompositeAlpha, 
		TranslatedWorldPosition, 
		RayDirection, 
		Roughness, 
		IndirectIrradiance, 
		IndirectSpecularOcclusion, 
		ExtraIndirectSpecular, 
		NumCulledReflectionCaptures, 
		CaptureDataStartIndex, 
		0,
		bCompositeSkylight);
}

float4 CompositeReflections(float4 ReflectionInput, float2 BufferUV, float InRoughness, uint ShadingModelID)
{
	float4 Lighting = float4(0, 0, 0, 1);

#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE
	const bool bHasBackfaceDiffuse = ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE || ShadingModelID == SHADINGMODELID_SUBSURFACE;
	const float FadeAlpha = ShadingModelID == SHADINGMODELID_CLEAR_COAT ? 1.0f : LumenCombineReflectionsAlpha(InRoughness, bHasBackfaceDiffuse);

	// Must branch as Lumen Reflections can be uninitialized where not needed and contain NaN
	if (FadeAlpha > 0.0f)
	{
		Lighting.rgb = ReflectionInput.xyz * FadeAlpha;
		Lighting.a = 1 - FadeAlpha;
	}

#else
	Lighting.rgb = ReflectionInput.rgb;
	Lighting.a = 1 - ReflectionInput.a;
#endif

	return Lighting;
}

float3 ReflectionEnvironment(FGBufferData GBuffer, float AmbientOcclusion, float2 BufferUV, float2 ScreenPosition, float4 SvPosition, float3 BentNormal, float3 SpecularColor, uint ShadingModelID)
{
	float4 Color = float4(0, 0, 0, 1);

	float IndirectIrradiance = GBuffer.IndirectIrradiance;
	
#if ENABLE_SKY_LIGHT && ALLOW_STATIC_LIGHTING
	BRANCH
	// Add in diffuse contribution from dynamic skylights so reflection captures will have something to mix with
	if (ReflectionStruct.SkyLightParameters.y > 0 && ReflectionStruct.SkyLightParameters.z > 0)
	{
		IndirectIrradiance += GetDynamicSkyIndirectIrradiance(BentNormal, GBuffer.WorldNormal);
	}
#endif

	float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), View.ScreenToTranslatedWorld).xyz;
	float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
	float3 ReflectionVector = reflect(CameraToPixel, GBuffer.WorldNormal);
    float3 V = -CameraToPixel;
	float3 N = GBuffer.WorldNormal;

	const float3 SavedTopLayerNormal = N;

#if SUPPORTS_ANISOTROPIC_MATERIALS
	ModifyGGXAnisotropicNormalRoughness(GBuffer.WorldTangent, GBuffer.Anisotropy, GBuffer.Roughness, N, V);
#endif

	float3 R = 2 * dot( V, N ) * N - V;
	float NoV = saturate( dot( N, V ) );

	// Point lobe in off-specular peak direction
	R = GetOffSpecularPeakReflectionDir(N, R, GBuffer.Roughness);

	// Sample input reflection texture that may contain SSR, planar reflections, RT reflections or Lumen Reflections

#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE
	float4 ReflectionInput = Texture2DArraySample(LumenReflectionTexture, LumenReflectionTextureSampler, float3(BufferUV, 0));
#elif REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_SSR
	float4 ReflectionInput = Texture2DSample(ReflectionTexture, ReflectionTextureSampler, BufferUV);
#else // REFLECTION_SOURCE_TILED_SSR
	const uint2 PixelPos = uint2(SvPosition.xy - View.ViewRectMin.xy);
	bool bSSRTileUsed = IsSSRTileUsed(PixelPos, SSRTiledViewRes, SSRTileMaskBuffer);
	float4 ReflectionInput = 0;

	if (bSSRTileUsed)
	{
		ReflectionInput = Texture2DSample(ReflectionTexture, ReflectionTextureSampler, BufferUV);
	}
#endif

	Color = CompositeReflections(ReflectionInput, BufferUV, GBuffer.Roughness, ShadingModelID);

	if(GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT )
	{
		const float ClearCoat = GBuffer.CustomData.x;
		Color = lerp( Color, float4(0,0,0,1), ClearCoat );

#if CLEAR_COAT_BOTTOM_NORMAL
		const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0/255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
		const float3 ClearCoatUnderNormal = OctahedronToUnitVector(oct1);

		const float3 BottomEffectiveNormal = ClearCoatUnderNormal;			
		R = 2 * dot( V, ClearCoatUnderNormal ) * ClearCoatUnderNormal - V;
#endif
	}

	float AO = GBuffer.GBufferAO * AmbientOcclusion;
	float RoughnessSq = GBuffer.Roughness * GBuffer.Roughness;
	float SpecularOcclusion = GetSpecularOcclusion(NoV, RoughnessSq, AO);
	Color.a *= SpecularOcclusion;

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;

	uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), GBuffer.Depth, 0);

	FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);
	uint NumCulledReflectionCaptures = CulledReflectionCapturesGridHeader.NumReflectionCaptures;
	uint CaptureDataStartIndex = CulledReflectionCapturesGridHeader.DataStartIndex;
#else
	uint CaptureDataStartIndex = 0;
	uint NumCulledReflectionCaptures = 0;
#endif

#if USE_ENERGY_CONSERVATION
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, NoV, GBuffer.SpecularColor);
#endif

	//Top of regular reflection or bottom layer of clear coat.
	Color.rgb += View.PreExposure * GatherRadiance(Color.a, TranslatedWorldPosition, R, GBuffer.Roughness, BentNormal, IndirectIrradiance, GBuffer.ShadingModelID, NumCulledReflectionCaptures, CaptureDataStartIndex);

	BRANCH
	if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
	{
		const float ClearCoat			= GBuffer.CustomData.x;
		const float ClearCoatRoughness	= GBuffer.CustomData.y;

		// Restore saved values needed for the top layer.
		GBuffer.WorldNormal = SavedTopLayerNormal;
		// Recompute some values unaffected by anistropy for the top layer
		N = GBuffer.WorldNormal;
		R = 2 * dot(V, N) * N - V;
		NoV = saturate(dot(N, V));
		R = GetOffSpecularPeakReflectionDir(N, R, ClearCoatRoughness);

		// TODO EnvBRDF should have a mask param
		#if USE_ENERGY_CONSERVATION
		Color.rgb *= EnergyTerms.E;
		#else
		// Hack: Ensures when clear coat is >0, grazing angle does not get too much energy, 
		//       but preserve response at normal incidence 
		float2 AB = PreIntegratedGF.SampleLevel(PreIntegratedGFSampler, float2(NoV, GBuffer.Roughness), 0).rg;
		Color.rgb *= SpecularColor * AB.x + AB.y * saturate(50 * SpecularColor.g) * (1 - ClearCoat);
		#endif

		// F_Schlick
		const float CoatF0 = 0.04f;
		#if USE_ENERGY_CONSERVATION
		float F = ComputeGGXSpecEnergyTerms(ClearCoatRoughness, NoV, CoatF0).E.x;
		#else
		float F = EnvBRDF(CoatF0, ClearCoatRoughness, NoV).x;
		#endif

		F *= ClearCoat;
			
		float LayerAttenuation = (1 - F);		
		Color.rgb *= LayerAttenuation;
		Color.a = F;
		
		Color.rgb += ReflectionInput.rgb * F;
		Color.a *= 1 - ReflectionInput.a;
			
		Color.a *= SpecularOcclusion;

		float3 TopLayerR = 2 * dot( V, N ) * N - V;
		Color.rgb += View.PreExposure * GatherRadiance(Color.a, TranslatedWorldPosition, TopLayerR, ClearCoatRoughness, BentNormal, IndirectIrradiance, GBuffer.ShadingModelID, NumCulledReflectionCaptures, CaptureDataStartIndex);

	}
	else
	{
		#if USE_ENERGY_CONSERVATION
		Color.rgb *= EnergyTerms.E;
		#else
		Color.rgb *= EnvBRDF( SpecularColor, GBuffer.Roughness, NoV );
		#endif
	}

	// Transform NaNs to black, transform negative colors to black.
	return -min(-Color.rgb, 0.0);
}

float GetCloudVolumetricAOShadow(in float3 TranslatedWorldPosition)
{
	// Ideally we would compute spatially how much of the sky+cloud is visible for each point in the world. But we evaluate the sky light only once at the sky light component position as of today.
	// To add some spatial variation, we can affect the sky light diffuse contribution according to cloud occlusion from above.
	// This is an approximation because clouds are also occluding the sky in the sky light capture (a bit of a double contribution then). but it does help by adding spatially varying details.
#if 0	// DISABLED for now because it has artefact with specular not being affected (and thus looking too bright which can be confusing for users).
	// NOTE: Consider wrapping this inside a define when enabled. Otherwise, perf will regress on less powerful platforms
	if (CloudSkyAOEnabled)
	{
		float OutOpticalDepth = 0.0f;
		return GetCloudVolumetricShadow(TranslatedWorldPosition, CloudSkyAOWorldToLightClipMatrix, CloudSkyAOFarDepthKm, CloudSkyAOTexture, CloudSkyAOSampler, OutOpticalDepth);
	}
#endif
	return 1.0f;
}

#if SUBSTRATE_ENABLED==1

struct FSSRContribution
{
	float3 SSRTopLayerEnvBRDF;
	float  SSRReductionFactor;
};

FSSRContribution InitSSRContribution()
{
	FSSRContribution Out;
	Out.SSRReductionFactor = 1.0f;
	Out.SSRTopLayerEnvBRDF = 0.0f;
	return Out;
}

void GetSSSRContribution(
	in FSubstrateEnvLightResult SubstrateEnvLight,
	in FSubstrateBSDF BSDF,
	in float3 BSDFThroughput,
	inout FSSRContribution Out)
{
	// The specular path weight applied on SSR. It must account for throughput even for top surface because it also contains coverage.
	Out.SSRTopLayerEnvBRDF += BSDFThroughput * SubstrateEnvLight.SSRSpecularWeight;

#if SUBSTRATE_FASTPATH==0
	if (BSDF_GETISTOPLAYER(BSDF) && BSDF_GETHASHAZINESS(BSDF) && any((SubstrateEnvLight.SpecularWeight + SubstrateEnvLight.SpecularHazeWeight) > 0.0f))
	{
		// SSR is traced for the sharpest lob. The smoothest one does not rely on SSR so we need to lower energy coming from SSR according to the lobe blend weight.
		// And we also try to make the transition smooth using Haziness
		Out.SSRReductionFactor -= dot(BSDFThroughput, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f)) * SubstrateEnvLight.SSRReduction;
	}
#endif 
}

float4 ApplySSRContribution(FSSRContribution In, float4 InReflectionInput)
{
	// Screen space reflections added on top of environment lighting.
	// Alpha is 0 to not affect the subsurface diffuse luma information.
	return float4(InReflectionInput.rgb * In.SSRTopLayerEnvBRDF * saturate(In.SSRReductionFactor), 0.0);
}

struct FLumenReflectionOnlyOutput
{
	float4 LightingLobe0;
	float4 LightingLobe1;
};

FLumenReflectionOnlyOutput GetLumenReflectionOnly(
	in FSubstrateEnvLightResult SubstrateEnvLight,
	in FSubstrateBSDF BSDF,
	in float3 BSDFThroughput,
	in float4 InReflectionInput)
{
	FLumenReflectionOnlyOutput Out = (FLumenReflectionOnlyOutput)0;
	Out.LightingLobe0.a = 1;
	Out.LightingLobe1.a = 1;
	
#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE

	const bool  bHasBackfaceDiffuse = SubstrateGetBSDFType(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && BSDF.HasBackScattering();

	// Primary highlight
	{
		const float Roughness = SubstrateGetBSDFRoughness(BSDF);
		const float FadeAlpha = LumenCombineReflectionsAlpha(Roughness, bHasBackfaceDiffuse);
		// Must branch as Lumen Reflections can be uninitialized where not needed and contain NaN
		if (FadeAlpha > 0.0f)
		{
			Out.LightingLobe0.rgb = InReflectionInput.xyz * FadeAlpha;
			Out.LightingLobe0.a = 1 - FadeAlpha;
		}
	}

	// Secondary highlight
	#if SUBSTRATE_FASTPATH==0
	if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
	{
		const float Roughness = SubstrateEnvLight.SpecularHazeSafeRoughness;
		const float FadeAlpha = LumenCombineReflectionsAlpha(Roughness, bHasBackfaceDiffuse);
		// Must branch as Lumen Reflections can be uninitialized where not needed and contain NaN
		if (FadeAlpha > 0.0f)
		{
			Out.LightingLobe1.rgb = InReflectionInput.xyz * FadeAlpha;
			Out.LightingLobe1.a = 1 - FadeAlpha;
		}
	}
	#endif 
#endif
	return Out;
}

#endif // SUBSTRATE_ENABLED==1

void ReflectionEnvironmentSkyLighting(
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	, out float3 OutOpaqueRoughRefractionSceneColor : SV_Target1
	, out float3 OutSubSurfaceSceneColor : SV_Target2
#elif SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE
	, out float OutSeparatedSubsurfaceDiffuse : SV_Target1
#endif
	)
{
	ResolvedView = ResolveView();

	uint2 PixelPos = SvPosition.xy;
	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;
	
	OutColor = 0.0f;
#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutOpaqueRoughRefractionSceneColor = 0.0f;
	OutSubSurfaceSceneColor = 0.0f;
#endif

#if SUBSTRATE_ENABLED==1

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bMaterialIsLit = SubstratePixelHeader.ClosureCount > 0;	// This test is also enough to exclude sky pixels
	#else
	const bool bForceUnlitOrDefaultLit = SUBSTRATE_FASTPATH;
	const FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV, true, bForceUnlitOrDefaultLit);
	FScreenSpaceData ScreenSpaceData;
	ScreenSpaceData.GBuffer = GBuffer;
	ScreenSpaceData.AmbientOcclusion = 1.0f; // Used, fetched from AmbientOcclusionTexture.
	const bool bMaterialIsLit = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
	#endif

	BRANCH
	if (bMaterialIsLit) // This test is also enough to exclude sky pixels
	{
		float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
		float SceneDepth = ConvertFromDeviceZ(DeviceZ);

		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
		float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
		float3 V = -CameraToPixel;

		// Sample the ambient occlusion that is dynamically generated every frame.
		float AmbientOcclusion = AmbientOcclusionTexture.SampleLevel(AmbientOcclusionSampler, BufferUV, 0).r;

	#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE
		const float TopLayerSpecularContributionFactor = 1.0f;
	#else
		float4 SSRReflection = 0.0f.xxxx;
		float TopLayerSpecularContributionFactor = 1.0f;

		#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_TILED_SSR
		if (IsSSRTileUsed(PixelPos, SSRTiledViewRes, SSRTileMaskBuffer))
		#endif
		{
			SSRReflection = Texture2DSample(ReflectionTexture, ReflectionTextureSampler, BufferUV);

			// SSR is computed for the top level by averaging BSDF components.
			// This is the top level specular contribution factor according to areas.
			TopLayerSpecularContributionFactor = 1.0f - SSRReflection.a;
		}

		// A reduction of SSR contribution is needed when there is haziness because we only consider SSR for the sharpest lobe.
		FSSRContribution SSRContribution = InitSSRContribution();
	#endif

		const float CloudVolumetricAOShadow = GetCloudVolumetricAOShadow(TranslatedWorldPosition);
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);

		FSubstrateDeferredLighting SubstrateLighting = GetInitialisedSubstrateDeferredLighting();
	
	#if APPLY_SKY_SHADOWING
		// Sample DFAO only once
		const float2 BentNormalUV = (SvPosition.xy - View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
		FUpsampleDFAOOutput UpsampleDFAOOutput = UpsampleDFAO(BentNormalUV, SceneDepth);
	#endif
	
		
	#if USE_DEFAULT_ENV_LIGHTING_INPUT && FEATURE_LEVEL >= FEATURE_LEVEL_SM5
		float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;
	
		uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SceneDepth, 0);

		FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);
		uint NumCulledReflectionCaptures = CulledReflectionCapturesGridHeader.NumReflectionCaptures;
		uint CaptureDataStartIndex = CulledReflectionCapturesGridHeader.DataStartIndex;
	#else
		uint CaptureDataStartIndex = 0;
		uint NumCulledReflectionCaptures = 0;
	#endif

		const bool bPixelMayHaveSSR = REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE || TopLayerSpecularContributionFactor < 1.0f;
		const bool bEnableSpecular = ReflectionStruct.SkyLightParameters.y > 0.0f || NumCulledReflectionCaptures > 0 || bPixelMayHaveSSR;

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
		Substrate_for (uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
	#else
		uint ClosureIndex = 0;
	#endif
		{
		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
			FSubstrateBSDF BSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V);
			float CombinedScreenAndMaterialAO = SubstrateGetAO(SubstratePixelHeader) * AmbientOcclusion;
			FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion = SubstrateGetIrradianceAndAO(SubstratePixelHeader);
		#else
			FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
			FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(GBufferBSDF.BSDFTangentBasis, GBufferBSDF.BSDF, V, V/*L*/, false/*bHasValidL*/);
			float CombinedScreenAndMaterialAO = 1.0; // AmbientOcclusion;
			FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion = SubstrateGetIrradianceAndAO(ScreenSpaceData.GBuffer);
		#endif
			const float3 BSDFThroughput = LuminanceWeight(SubstrateBSDFContext, SubstrateBSDFContext.BSDF); // Use the reflected direction
	
			// Evaluate environment lighting
			FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, bEnableSpecular, Settings);
	
			float3 BentNormal = SubstrateEnvLight.DiffuseNormal;
		#if APPLY_SKY_SHADOWING
			// Set DiffuseNormal as the bent normal for all diffuse computations.
			BentNormal = ApplyDFAO(UpsampleDFAOOutput, SubstrateEnvLight.DiffuseNormal);
		#endif
	
			float3 DiffuseLighting = 0;
			float3 SpecularLighting = 0;
			SubstrateEnvLightingCommon(
				SubstrateEnvLight,
				SubstrateIrradianceAndOcclusion,
				SubstrateBSDFContext,
				SubstrateBSDFContext.BSDF,
				BentNormal,
				BSDFThroughput,
				CaptureDataStartIndex,
				NumCulledReflectionCaptures,
				AmbientOcclusion,
				CloudVolumetricAOShadow,
				TopLayerSpecularContributionFactor,
				TranslatedWorldPosition,
				CombinedScreenAndMaterialAO,
				DiffuseLighting,
				SpecularLighting);

		#if REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE
			{
				// Pre exposure the sky light specular evaluated from within SubstrateEnvLightingCommon (CompositeReflectionCapturesAndSkylightTWS) when rendering lumen HWRT reflections while diffuse global illumination is "none".
				// This matches with the fact that LumenReflectionTexture is also already pre-exposed, see condition below using "REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE".
				SpecularLighting *= View.PreExposure; 

				// Diffuse lighting needs pre exposition as GetEnvDiffuseLighting() is not preexposed
				DiffuseLighting *= View.PreExposure; 

				const FLumenReflectionOnlyOutput LumenReflections = GetLumenReflectionOnly(SubstrateEnvLight, SubstrateBSDFContext.BSDF, BSDFThroughput, Texture2DArraySample(LumenReflectionTexture, LumenReflectionTextureSampler, float3(BufferUV, ClosureIndex)));

				SpecularLighting *= min(LumenReflections.LightingLobe0.a, LumenReflections.LightingLobe1.a);

				// Primary specular lobe
				{
					SpecularLighting += LumenReflections.LightingLobe0.xyz * BSDFThroughput * SubstrateEnvLight.SpecularWeight;
				}

				// Secondary specular lobe
				#if SUBSTRATE_FASTPATH==0
				if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
				{
					SpecularLighting += LumenReflections.LightingLobe1.xyz * BSDFThroughput * SubstrateEnvLight.SpecularHazeWeight;
				}
				#endif 
			}
		#else
			if (BSDF_GETISTOPLAYER(SubstrateBSDFContext.BSDF))
			{
				GetSSSRContribution(SubstrateEnvLight, SubstrateBSDFContext.BSDF, BSDFThroughput, SSRContribution);
			}
		#endif

		#if SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION
			SpecularLighting = SubstrateToonQuantize(SubstrateEnvLight.ToonProfileID, SpecularLighting);
		#endif

			FLightAccumulator Out = (FLightAccumulator)0;
			LightAccumulator_AddSplit(Out, DiffuseLighting, SpecularLighting, DiffuseLighting, 1.f /*CommonMultiplier*/, SubstrateEnvLight.bPostProcessSubsurface);
			AccumulateSubstrateDeferredLighting(SubstrateLighting, Out, SubstrateEnvLight.bPostProcessSubsurface, BSDF_GETISTOPLAYER(SubstrateBSDFContext.BSDF));
		}

		OutColor = SubstrateLighting.SceneColor * (REFLECTION_SOURCE_TYPE == REFLECTION_SOURCE_LUMEN_STANDALONE ? 1.0f : View.PreExposure);
		#if REFLECTION_SOURCE_TYPE != REFLECTION_SOURCE_LUMEN_STANDALONE
		OutColor += ApplySSRContribution(SSRContribution, SSRReflection);
		#endif

		#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		OutOpaqueRoughRefractionSceneColor += SubstrateLighting.OpaqueRoughRefractionSceneColor;
		OutSubSurfaceSceneColor += SubstrateLighting.SubSurfaceSceneColor;

		OutOpaqueRoughRefractionSceneColor.rgb *= View.PreExposure;
		OutSubSurfaceSceneColor.rgb *= View.PreExposure;
		#endif
	}

#else // SUBSTRATE_ENABLED==1

	// Sample scene textures.
	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);

	uint ShadingModelID = GBuffer.ShadingModelID;
	const bool bUnlitMaterial = ShadingModelID == SHADINGMODELID_UNLIT;

	float3 DiffuseColor = GBuffer.DiffuseColor;
	float3 SpecularColor = GBuffer.SpecularColor;
	RemapClearCoatDiffuseAndSpecularColor(GBuffer, ScreenPosition, DiffuseColor, SpecularColor);

	// Sample the ambient occlusion that is dynamically generated every frame.
	float AmbientOcclusion = AmbientOcclusionTexture.SampleLevel(AmbientOcclusionSampler, BufferUV, 0).r;

	float3 BentNormal = GBuffer.WorldNormal;
#if APPLY_SKY_SHADOWING
	{
		const float2 BentNormalUV = (SvPosition.xy - View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
		BentNormal = UpsampleDFAO(BentNormalUV, GBuffer.Depth, GBuffer.WorldNormal);
	}
#endif

#if ENABLE_DYNAMIC_SKY_LIGHT

	BRANCH
	if (!bUnlitMaterial) // Only light pixels marked as lit
	{
		float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), View.ScreenToTranslatedWorld).xyz;
		const float CloudVolumetricAOShadow = GetCloudVolumetricAOShadow(TranslatedWorldPosition);

		float3 SkyLighting = CloudVolumetricAOShadow * SkyLightDiffuse(GBuffer, AmbientOcclusion, BufferUV, ScreenPosition, BentNormal, DiffuseColor);

		FLightAccumulator LightAccumulator = (FLightAccumulator)0;
		const bool bNeedsSeparateSubsurfaceLightAccumulation = UseSubsurfaceProfile(ShadingModelID);
		LightAccumulator_Add(LightAccumulator, SkyLighting, SkyLighting, 1.0f, bNeedsSeparateSubsurfaceLightAccumulation);
		OutColor = LightAccumulator_GetResult(LightAccumulator);
	}

#endif // ENABLE_DYNAMIC_SKY_LIGHT 

	BRANCH
	if (!bUnlitMaterial && ShadingModelID != SHADINGMODELID_HAIR)
	{
		OutColor.xyz += ReflectionEnvironment(GBuffer, AmbientOcclusion, BufferUV, ScreenPosition, SvPosition, BentNormal, SpecularColor, ShadingModelID);
	}

#endif // SUBSTRATE_ENABLED==1
	
#if SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE && !SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	OutSeparatedSubsurfaceDiffuse = EncodeSeparatedSubsurfaceDiffuse(OutColor);
#endif
}

