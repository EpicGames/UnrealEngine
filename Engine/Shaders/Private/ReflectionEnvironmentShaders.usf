// Copyright Epic Games, Inc. All Rights Reserved.

/*============================================================================= 

=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "BRDF.ush"

#if defined(DXC_GROUPSHARED_ALIGNMENT_WORKAROUND) && DXC_GROUPSHARED_ALIGNMENT_WORKAROUND
// Workaround for shader compiler bug affecting certain platforms
// Align groupshared SH data if the shader compiler requires it
#define NEED_SH_VECTOR_PADDING 1
#endif
#include "SHCommon.ush"
#include "ReflectionEnvironmentShared.ush"
#include "MonteCarlo.ush"
#include "SkyLightingShared.ush"

struct FCopyToCubeFaceVSOutput
{
	float2 UV : TEXCOORD0;
	float3 ScreenVector : TEXCOORD1;
	float4 Position : SV_POSITION;
};

void CopyToCubeFaceVS(
	in float2 InPosition : ATTRIBUTE0,
	in float2 InUV       : ATTRIBUTE1,
	out FCopyToCubeFaceVSOutput Out
	)
{	
	DrawRectangle(float4(InPosition.xy, 0, 1), InUV, Out.Position, Out.UV);
	Out.ScreenVector = ScreenVectorFromScreenRect(float4(Out.Position.xy, 1, 0));
}

int CubeFace;

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

#if !SHADING_PATH_DEFERRED
Texture2D SceneDepthTexture;
#endif
SamplerState SceneDepthSampler;

/** 
 * X = 0 if capturing sky light, 1 if capturing reflection capture with MaxDistance fade, 2 otherwise, 
 * Y = Sky distance threshold, 
 * Z = whether a skylight's lower hemisphere should be replaced with LowerHemisphereColor.
 */
float4 SkyLightCaptureParameters;
float4 LowerHemisphereColor;
float4 ColorAlphaMultiplier;
float ClampToFP16;

void CopySceneColorToCubeFaceColorPS(
	FCopyToCubeFaceVSOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	float SceneDepth = SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#if !SCENE_TEXTURES_DISABLED
	SceneDepth = ConvertFromDeviceZ(Texture2DSample(SceneDepthTexture, SceneDepthSampler, Input.UV).r);
#endif
	float3 SceneColor = Texture2DSample(SceneColorTexture, SceneColorSampler, Input.UV).rgb;

	// Convert INF's to valid values
	SceneColor = ClampToHalfFloatRange(SceneColor);
	float3 TranslatedWorldPosition = Input.ScreenVector * SceneDepth + View.TranslatedWorldCameraOrigin;

	float Alpha = 1;

	if (SkyLightCaptureParameters.x == 0)
	{
		// Assuming we're on a planet and no sky lighting is coming from below the horizon
		// This is important to avoid leaking from below since we are integrating incoming lighting and shadowing separately
		if (Input.ScreenVector.z < 0 && SkyLightCaptureParameters.z >= 1)
		{
			SceneColor = lerp(SceneColor, LowerHemisphereColor.rgb, LowerHemisphereColor.a);
		}
	}
	else if (SkyLightCaptureParameters.x == 1)
	{
		float RadialDistance = length(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);
		float MaxDistance = SkyLightCaptureParameters.y;

		// Setup alpha to fade out smoothly past the max distance
		// This allows a local reflection capture to only provide reflections where it has valid data, falls back to sky cubemap
		Alpha = 1 - smoothstep(.8f * MaxDistance, MaxDistance, RadialDistance);
	}

	// We need pre-multiplied alpha for correct filtering
	// However, we need to compute average brightness before masking out sky areas, so premultiplying happens later
	OutColor = float4(SceneColor, Alpha);
}

float3 GetCubemapVector(float2 ScaledUVs, int InCubeFace)
{
	float3 CubeCoordinates;

	//@todo - this could be a 3x3 matrix multiply
	if (InCubeFace == 0)
	{
		CubeCoordinates = float3(1, -ScaledUVs.y, -ScaledUVs.x);
	}
	else if (InCubeFace == 1)
	{
		CubeCoordinates = float3(-1, -ScaledUVs.y, ScaledUVs.x);
	}
	else if (InCubeFace == 2)
	{
		CubeCoordinates = float3(ScaledUVs.x, 1, ScaledUVs.y);
	}
	else if (InCubeFace == 3)
	{
		CubeCoordinates = float3(ScaledUVs.x, -1, -ScaledUVs.y);
	}
	else if (InCubeFace == 4)
	{
		CubeCoordinates = float3(ScaledUVs.x, -ScaledUVs.y, 1);
	}
	else
	{
		CubeCoordinates = float3(-ScaledUVs.x, -ScaledUVs.y, -1);
	}

	return CubeCoordinates;
}

void GetCubemapTangent(int InCubeFace, inout float3 OutX, inout float3 OutY)
{
	if (InCubeFace == 0)
	{
		OutX = float3(0,-1, 0);
		OutY = float3(0, 0,-1);
	}
	else if (InCubeFace == 1)
	{
		OutX = float3(0,-1, 0);
		OutY = float3(0, 0, 1);
	}
	else if (InCubeFace == 2)
	{
		OutX = float3(1, 0, 0);
		OutY = float3(0, 0, 1);
	}
	else if (InCubeFace == 3)
	{
		OutX = float3(1, 0, 0);
		OutY = float3(0, 0,-1);
	}
	else if (InCubeFace == 4)
	{
		OutX = float3(1, 0, 0);
		OutY = float3(0,-1, 0);
	}
	else
	{
		OutX = float3(-1, 0, 0);
		OutY = float3( 0,-1, 0);
	}
}

float CubeTexelWeight( float3 N )
{
	uint Axis = 2;
	if( abs(N.x) >= abs(N.y) && abs(N.x) >= abs(N.z) )
	{
		Axis = 0;
	}
	else if( abs(N.y) > abs(N.z) )
	{
		Axis = 1;
	}

	N = Axis == 0 ? N.zyx : N;
	N = Axis == 1 ? N.xzy : N;

	float2 UV = N.xy / N.z;

	float VecLengthSqr = 1 + dot( UV, UV );
	return 4.0 / ( sqrt( VecLengthSqr ) * VecLengthSqr );
}


TextureCube SourceCubemapTexture;
SamplerState SourceCubemapSampler;
float2 SinCosSourceCubemapRotation;
float2 SvPositionToUVScale;

void CopyCubemapToCubeFaceColorPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
	// Rotate around Z axis
	CubeCoordinates.xy = float2(dot(CubeCoordinates.xy, float2(SinCosSourceCubemapRotation.y, -SinCosSourceCubemapRotation.x)), dot(CubeCoordinates.xy, SinCosSourceCubemapRotation));
	OutColor = TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, CubeCoordinates, 0);

	if (SkyLightCaptureParameters.x > 0)
	{
		// Assuming we're on a planet and no sky lighting is coming from below the horizon
		// This is important to avoid leaking from below since we are integrating incoming lighting and shadowing separately
		if (CubeCoordinates.z < 0 && SkyLightCaptureParameters.z >= 1)
		{
			OutColor.rgb = lerp(OutColor.rgb, LowerHemisphereColor.rgb, LowerHemisphereColor.a);
		}
	}

	OutColor.a = 1;

	OutColor *= ColorAlphaMultiplier;

	OutColor = ClampToFP16 == 0.0f ? OutColor : clamp(OutColor, float4(0.0f, 0.0f, 0.0f, 0.0f), float4(MaxHalfFloat, MaxHalfFloat, MaxHalfFloat, 1.0f));
}

void CopyCubemapToIlluminanceMeterCubemapPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();

	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;

	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);

	float NoL = saturate(normalize(CubeCoordinates).z);
	//float NoL = CubeCoordinates.z > 0.0 ? 1 : 0;

	// See https://www.ppsloan.org/publications/StupidSH36.pdf, p.9. 
	float Tmp= 1.0f + ScaledUVs.x * ScaledUVs.x + ScaledUVs.y * ScaledUVs.y; 
	float dA = 4.0f / (sqrt(Tmp) * Tmp); // Not really dA, but using same variable name

	const float3 L = TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, CubeCoordinates, 0).rgb * ResolvedView.SkyLightColor.rgb;
	OutColor = float4(L * NoL* dA, dA);
}

uint MipIndex;
uint NumMips;
uint SourceMipIndex;

float4 SampleCubemap(float3 Coordinates, uint InMipIndex)
{
	return TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, Coordinates, InMipIndex);
}

float4 SampleCubemap(float3 Coordinates)
{
#ifdef USE_COMPUTE
	return SampleCubemap(Coordinates, 0);
#else
	return SampleCubemap(Coordinates, SourceMipIndex);
#endif
}

#ifdef USE_COMPUTE
int FaceThreadGroupSize;
int2 ValidDispatchCoord;
RWTexture2DArray<float4> OutTextureMipColor;	// All slices, i.e. faces, of a cube map mip level
#endif 

#ifdef USE_COMPUTE
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void DownsampleCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const uint2 FaceCoord = uint2(ThreadId.x % uint(FaceThreadGroupSize), ThreadId.y);
	if (any(FaceCoord >= uint2(ValidDispatchCoord)))
	{
		return;
	}
	const int SelectedCubeFace = int(ThreadId.x) / FaceThreadGroupSize;
	float2 ScaledUVs = ((float2(FaceCoord) + 0.5f) / float2(ValidDispatchCoord)) * 2.0f - 1.0f;
	float4 OutColor;
#else // USE_COMPUTE
void DownsamplePS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{ 
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	const int SelectedCubeFace = CubeFace;
#endif // USE_COMPUTE
	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, SelectedCubeFace);

	uint MipSize = 1u << ( NumMips - MipIndex - 1 );

	float3 TangentZ = normalize( CubeCoordinates );
	float3 TangentX = normalize( cross( GetCubemapVector( ScaledUVs + float2(0,1), SelectedCubeFace), TangentZ ) );
	float3 TangentY = cross( TangentZ, TangentX );

	const float SampleOffset = 2.0 * 2 / MipSize;

	float2 Offsets[] =
	{
		float2(-1, -1) * 0.7,
		float2( 1, -1) * 0.7,
		float2(-1,  1) * 0.7,
		float2( 1,  1) * 0.7,
		
		float2( 0, -1),
		float2(-1,  0),
		float2( 1,  0),
		float2( 0,  1),
	};

	OutColor = SampleCubemap(CubeCoordinates);

	UNROLL
	for( uint i = 0; i < 8; i++ )
	{
		float Weight = 0.375;

		float3 SampleDir = CubeCoordinates;
		SampleDir += TangentX * ( Offsets[i].x * SampleOffset );
		SampleDir += TangentY * ( Offsets[i].y * SampleOffset );
		OutColor += SampleCubemap(SampleDir) * Weight;
	}

	OutColor *= rcp( 1.0 + 1.0 + 2.0 );

#ifdef USE_COMPUTE
	OutTextureMipColor[uint3(FaceCoord, SelectedCubeFace)] = OutColor;
#endif
}

void DownsampleMaxPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
)
{
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	const int SelectedCubeFace = CubeFace;

	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, SelectedCubeFace);

	uint MipSize = 1u << (NumMips - MipIndex - 1);

	float3 TangentX = 0.0f;
	float3 TangentY = 0.0f;
	GetCubemapTangent(SelectedCubeFace, TangentX, TangentY);

	const float SampleOffset = 1.0f / MipSize;

	// Offset from the connect of 4 texels towards their center.
	float2 Offsets[] =
	{
		float2(-0.5,-0.5),
		float2( 0.5,-0.5),
		float2( 0.5, 0.5),
		float2(-0.5, 0.5),
	};

	OutColor = 0;

	UNROLL
	for (uint i = 0; i < 4; i++)
	{
		float3 SampleDir = CubeCoordinates;
		SampleDir += TangentX * (Offsets[i].x * SampleOffset);
		SampleDir += TangentY * (Offsets[i].y * SampleOffset);
		OutColor = max(OutColor, SourceCubemapTexture.SampleLevel(SourceCubemapSampler, SampleDir, 0));
	}
}

void DownsampleIntegrateCubemapIlluminancePS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
)
{
	OutColor = 0;
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	const int SelectedCubeFace = CubeFace;

	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, SelectedCubeFace);

	uint MipSize = 1u << (NumMips - MipIndex - 1);

	float3 TangentX = 0.0f;
	float3 TangentY = 0.0f;
	GetCubemapTangent(SelectedCubeFace, TangentX, TangentY);

	const float SampleOffset = 1.0f / MipSize;

	// Offset from the connect of 4 texels towards their center.
	float2 Offsets[] =
	{
		float2(-0.5,-0.5),
		float2( 0.5,-0.5),
		float2( 0.5, 0.5),
		float2(-0.5, 0.5),
	};

	OutColor = 0;

	UNROLL
	for (uint i = 0; i < 4; i++)
	{
		float3 SampleDir = CubeCoordinates;
		SampleDir += TangentX * (Offsets[i].x * SampleOffset);
		SampleDir += TangentY * (Offsets[i].y * SampleOffset);
		OutColor += SourceCubemapTexture.SampleLevel(SourceCubemapSampler, SampleDir, 0);
	}
}



#if APPLY_LOWER_HEMISPHERE_COLOR_PIXELSHADER

int ApplyLowerHemisphereColor;
float4 LowerHemisphereSolidColor;

int ApplyLowResCloudTexture;
Texture2D<float4> LowResCloudTexture;
SamplerState LowResCloudSampler;

// This pixel shader is designed to work with RenderRealTimeReflectionHeightFogVS
void ApplyLowerHemisphereColorPS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminanceAlpha : SV_Target0)
{
	ResolvedView = ResolveView();

	float2 UV = SVPos.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;

	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
	float3 N = normalize(CubeCoordinates);

	OutLuminanceAlpha = float4(0.0f, 0.0f, 0.0f, 1.0f);

	if (ApplyLowResCloudTexture > 0)
	{
		OutLuminanceAlpha = LowResCloudTexture.Sample(LowResCloudSampler, UV);
	}

	if (N.z < 0.0f && ApplyLowerHemisphereColor > 0)
	{
		float Coverage = 0.0f;
		Coverage = pow(saturate(LowerHemisphereSolidColor.a), 2.2);
		Coverage = saturate(LowerHemisphereSolidColor.a);
		const float Exposure = ResolvedView.RealTimeReflectionCapture ? ResolvedView.RealTimeReflectionCapturePreExposure : ResolvedView.PreExposure;
		OutLuminanceAlpha = float4(lerp(OutLuminanceAlpha.rgb, LowerHemisphereSolidColor.rgb * Exposure, Coverage), OutLuminanceAlpha.a * (1.0 - Coverage));
	}
}

#endif



int NumCaptureArrayMips;

/** Cube map array of reflection captures. */
TextureCube ReflectionEnvironmentColorTexture;
SamplerState ReflectionEnvironmentColorSampler;

#if COMPUTEBRIGHTNESS_PIXELSHADER

void ComputeBrightnessMain(out float4 OutColor : SV_Target0)
{ 
	// Sample the 6 1x1 cube faces and average
	float3 AverageColor = TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(1, 0, 0), NumCaptureArrayMips - 1).rgb;
	AverageColor += TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(-1, 0, 0), NumCaptureArrayMips - 1).rgb;
	AverageColor += TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(0, 1, 0), NumCaptureArrayMips - 1).rgb;
	AverageColor += TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(0, -1, 0), NumCaptureArrayMips - 1).rgb;
	AverageColor += TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(0, 0, 1), NumCaptureArrayMips - 1).rgb;
	AverageColor += TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(0, 0, -1), NumCaptureArrayMips - 1).rgb;

	OutColor = dot(AverageColor / 6, .3333f); 
}

#endif

#if COMPUTEMAXLUMINANCE_PIXELSHADER

void ComputeCubeMaxLuminancePS(out float4 OutColor : SV_Target0)
{ 
	// Sample the 6 1x1 cube faces and average
	float3 MaxLuminanceRGB = TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(1, 0, 0), NumCaptureArrayMips - 1).rgb;
	MaxLuminanceRGB = max(MaxLuminanceRGB, TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3(-1, 0, 0), NumCaptureArrayMips - 1).rgb);
	MaxLuminanceRGB = max(MaxLuminanceRGB, TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3( 0, 1, 0), NumCaptureArrayMips - 1).rgb);
	MaxLuminanceRGB = max(MaxLuminanceRGB, TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3( 0,-1, 0), NumCaptureArrayMips - 1).rgb);
	MaxLuminanceRGB = max(MaxLuminanceRGB, TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3( 0, 0, 1), NumCaptureArrayMips - 1).rgb);
	MaxLuminanceRGB = max(MaxLuminanceRGB, TextureCubeSampleLevel(ReflectionEnvironmentColorTexture, ReflectionEnvironmentColorSampler, float3( 0, 0,-1), NumCaptureArrayMips - 1).rgb);

	float MaxLuminance = max(MaxLuminanceRGB.r, max(MaxLuminanceRGB.g, MaxLuminanceRGB.b));
	OutColor = MaxLuminance;
}

#endif

#if SHADING_PATH_MOBILE
// There's a precision problem in OpenGL because the default precision in the Pixel Shader is set to half, even for constant values, use Hammersley16 instead.
#define HammersleyDistribution Hammersley16
#else
#define HammersleyDistribution Hammersley
#endif

#ifdef USE_COMPUTE
int CubeFaceOffset;
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void FilterCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const uint2 FaceCoord = uint2(ThreadId.x % uint(FaceThreadGroupSize), ThreadId.y);
	if (any(FaceCoord >= uint2(ValidDispatchCoord)))
	{
		return;
	}
	const int SelectedCubeFace = CubeFaceOffset + int(ThreadId.x) / FaceThreadGroupSize;
	float2 ScaledUVs = ((float2(FaceCoord) + 0.5f) / float2(ValidDispatchCoord)) * 2.0f - 1.0f;
	float4 OutColor;
#else // USE_COMPUTE
void FilterPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{ 
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	const int SelectedCubeFace = CubeFace;
#endif // USE_COMPUTE
	float3 CubeCoordinates = GetCubemapVector(ScaledUVs, SelectedCubeFace);

	float3 N = normalize(CubeCoordinates);
	float3x3 TangentToWorld = GetTangentBasis( N );

	float Roughness = ComputeReflectionCaptureRoughnessFromMip( MipIndex, NumMips - 1 );

	if( Roughness < 0.01 )
	{
		OutColor = SourceCubemapTexture.SampleLevel(SourceCubemapSampler, CubeCoordinates, 0 );
		
		// Prevent NaNs from going into the cube map
		OutColor = -min(-OutColor, 0);
		
	#ifdef USE_COMPUTE
		OutTextureMipColor[uint3(FaceCoord, SelectedCubeFace)] = OutColor;
	#endif
		return;
	}

	uint CubeSize = 1u << ( NumMips - 1 );
	const float SolidAngleTexel = 4.0f * PI / float(6.0f * CubeSize * CubeSize) * 2.0f;

#if 0
	// Reference
	const uint NumSamples = 1024;
#elif SHADING_PATH_MOBILE
	// Mobile path - a bit lower quality
	uint NumSamples = 0;
	if (Roughness <= 0.1)
	{
		NumSamples = 16;
	}
	else if (Roughness <= 0.4)
	{
		NumSamples = 32;
	}
	else
	{
		NumSamples = 64;
	}
#else
	// General real time
	const uint NumSamples = Roughness < 0.1 ? 32 : 64;
#endif
	
	float4 FilteredColor = 0;
	BRANCH
	if( Roughness > 0.99 )
	{
		// Roughness=1, GGX is constant. Use cosine distribution instead

		LOOP
		for( uint i = 0; i < NumSamples; i++ )
		{
			float2 E = HammersleyDistribution( i, NumSamples, 0 );

			float3 L = CosineSampleHemisphere( E ).xyz;

			float NoL = L.z;

			float PDF = NoL / PI;
			float SolidAngleSample = 1.0 / ( NumSamples * PDF );
			float Mip = 0.5 * log2( SolidAngleSample / SolidAngleTexel );

			L = mul( L, TangentToWorld );
			FilteredColor += SourceCubemapTexture.SampleLevel(SourceCubemapSampler, L, Mip );
		}

		OutColor = FilteredColor / NumSamples;
	}
	else
	{
		float Weight = 0;

		LOOP
		for( uint i = 0; i < NumSamples; i++ )
		{
			float2 E = HammersleyDistribution( i, NumSamples, 0 );

			// 6x6 Offset rows. Forms uniform star pattern
			//uint2 Index = uint2( i % 6, i / 6 );
			//float2 E = ( Index + 0.5 ) / 5.8;
			//E.x = frac( E.x + (Index.y & 1) * (0.5 / 6.0) );

			E.y *= 0.995;

			float3 H = ImportanceSampleGGX( E, Pow4(Roughness) ).xyz;
			float3 L = 2 * H.z * H - float3(0,0,1);

			float NoL = L.z;
			float NoH = H.z;

			if( NoL > 0 )
			{
				//float TexelWeight = CubeTexelWeight( L );
				//float SolidAngleTexel = SolidAngleAvgTexel * TexelWeight;

				//float PDF = D_GGX( Pow4(Roughness), NoH ) * NoH / (4 * VoH);
				float PDF = D_GGX( Pow4(Roughness), NoH ) * 0.25;
				float SolidAngleSample = 1.0 / ( NumSamples * PDF );
				float Mip = 0.5 * log2( SolidAngleSample / SolidAngleTexel );

				float ConeAngle = acos( 1 - SolidAngleSample / (2*PI) );

				L = mul( L, TangentToWorld );
				FilteredColor += SourceCubemapTexture.SampleLevel(SourceCubemapSampler, L, Mip ) * NoL;
				Weight += NoL;
			}
		}

		OutColor = FilteredColor / Weight;
	}

	// Prevent NaNs from going into the cube map
	OutColor = -min(-OutColor, 0);

#ifdef USE_COMPUTE
	OutTextureMipColor[uint3(FaceCoord, SelectedCubeFace)] = OutColor;
#endif
}

float4 CoefficientMask0;
float4 CoefficientMask1;
float CoefficientMask2;
int NumSamples;

void DiffuseIrradianceCopyPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{ 
	float2 UV = SvPosition.xy * SvPositionToUVScale;
	float2 ScaledUVs = UV * 2 - 1;
	float3 CubeCoordinates = normalize(GetCubemapVector(ScaledUVs, CubeFace));

	float SquaredUVs = 1 + dot(ScaledUVs, ScaledUVs);
	// Dividing by NumSamples here to keep the sum in the range of fp16, once we get down to the 1x1 mip
	float TexelWeight = 4 / (sqrt(SquaredUVs) * SquaredUVs);

	FThreeBandSHVector SHCoefficients = SHBasisFunction3(CubeCoordinates);
	float CurrentSHCoefficient = dot(SHCoefficients.V0, CoefficientMask0) + dot(SHCoefficients.V1, CoefficientMask1) + SHCoefficients.V2 * CoefficientMask2;
	float3 TexelLighting = SampleCubemap(CubeCoordinates).rgb;

	OutColor = float4(TexelLighting * CurrentSHCoefficient * TexelWeight, TexelWeight);
}

float4 Sample01;
float4 Sample23;

void DiffuseIrradianceAccumulatePS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
	)
{ 
	float4 AccumulatedValue = 0;
	float2 UV = SvPosition.xy * SvPositionToUVScale;

	{
		float2 ScaledUVs = saturate(UV + Sample01.xy) * 2 - 1;
		float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
		AccumulatedValue += SampleCubemap(CubeCoordinates);
	}
	
	{
		float2 ScaledUVs = saturate(UV + Sample01.zw) * 2 - 1;
		float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
		AccumulatedValue += SampleCubemap(CubeCoordinates);
	}

	{
		float2 ScaledUVs = saturate(UV + Sample23.xy) * 2 - 1;
		float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
		AccumulatedValue += SampleCubemap(CubeCoordinates);
	}

	{
		float2 ScaledUVs = saturate(UV + Sample23.zw) * 2 - 1;
		float3 CubeCoordinates = GetCubemapVector(ScaledUVs, CubeFace);
		AccumulatedValue += SampleCubemap(CubeCoordinates);
	}
	
	OutColor = float4(AccumulatedValue.rgb / 4.0f, AccumulatedValue.a / 4.0f);
}

void AccumulateCubeFacesPS(
	out float4 OutColor : SV_Target0
	)
{
	float4 AccumulatedValue; 
	AccumulatedValue  = TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(1, 0, 0), SourceMipIndex);
	AccumulatedValue += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(-1, 0, 0), SourceMipIndex);
	AccumulatedValue += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(0, 1, 0), SourceMipIndex);
	AccumulatedValue += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(0, -1, 0), SourceMipIndex);
	AccumulatedValue += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(0, 0, 1), SourceMipIndex);
	AccumulatedValue += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(0, 0, -1), SourceMipIndex);
	OutColor = float4(4 * PI * AccumulatedValue.rgb / ( max(AccumulatedValue.a, .00001f)), 0);	
}


RWTexture2DArray<float4> OutCubeFaces;
Texture2D SHTexture;
float MipInvSize;

float3 GetCubeVector(uint3 DispatchThreadId)
{
	float2 UV = (DispatchThreadId.xy + 0.5) * MipInvSize;
	UV = UV * 2.0 - 1.0;

	float3 CubeVector;
	if (DispatchThreadId.z == 0)
	{
		CubeVector = float3(1, -UV.y, -UV.x);
	}
	else if (DispatchThreadId.z == 1)
	{
		CubeVector = float3(-1, -UV.y, UV.x);
	}
	else if (DispatchThreadId.z == 2)
	{
		CubeVector = float3(UV.x, 1, UV.y);
	}
	else if (DispatchThreadId.z == 3)
	{
		CubeVector = float3(UV.x, -1, -UV.y);
	}
	else if (DispatchThreadId.z == 4)
	{
		CubeVector = float3(UV.x, -UV.y, 1);
	}
	else if (DispatchThreadId.z == 5)
	{
		CubeVector = float3(-UV.x, -UV.y, -1);
	}
	return normalize(CubeVector);
}

// Computes diffuse irradiance from SH coefficients
[numthreads(8, 8, 1)]
void SHCoefficientsToCubemapCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	float3 CubeVector = GetCubeVector(DispatchThreadId);
	
	float4 Coefficients[9] =
	{
		SHTexture.Load(int3(0, 0, 0)),
		SHTexture.Load(int3(1, 0, 0)),
		SHTexture.Load(int3(2, 0, 0)),
		SHTexture.Load(int3(3, 0, 0)),
		SHTexture.Load(int3(4, 0, 0)),
		SHTexture.Load(int3(5, 0, 0)),
		SHTexture.Load(int3(6, 0, 0)),
		SHTexture.Load(int3(7, 0, 0)),
		SHTexture.Load(int3(8, 0, 0))
	};

	FThreeBandSHVectorRGB SHVector;
	SHVector.R.V0 = half4(Coefficients[0].r, Coefficients[1].r, Coefficients[2].r, Coefficients[3].r);
	SHVector.R.V1 = half4(Coefficients[4].r, Coefficients[5].r, Coefficients[6].r, Coefficients[7].r);
	SHVector.R.V2 = Coefficients[8].r;
	SHVector.G.V0 = half4(Coefficients[0].g, Coefficients[1].g, Coefficients[2].g, Coefficients[3].g);
	SHVector.G.V1 = half4(Coefficients[4].g, Coefficients[5].g, Coefficients[6].g, Coefficients[7].g);
	SHVector.G.V2 = Coefficients[8].g;
	SHVector.B.V0 = half4(Coefficients[0].b, Coefficients[1].b, Coefficients[2].b, Coefficients[3].b);
	SHVector.B.V1 = half4(Coefficients[4].b, Coefficients[5].b, Coefficients[6].b, Coefficients[7].b);
	SHVector.B.V2 = Coefficients[8].b;

	FThreeBandSHVector SHBasis = SHBasisFunction3(CubeVector);

	OutCubeFaces[DispatchThreadId] = half4(DotSH3(SHVector, SHBasis), 1.0);
}

Texture2DArray<float4> InCubeFaces;
int InMipSizeMinusOne;
float InMipInvSize;

// Reference shader that directly computes diffuse irradiance from a source cubemap
[numthreads(8, 8, 1)]
void DiffuseIrradianceReferenceCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	float3 Normal = GetCubeVector(DispatchThreadId);
	float3 Irradiance = float3(0.0, 0.0, 0.0);
	float TotalWeight = 0.0;

	for (int IndexY = 0; IndexY <= InMipSizeMinusOne; IndexY++)
	{
		for (int IndexX = 0; IndexX <= InMipSizeMinusOne; IndexX++)
		{
			float2 UV = (float2(IndexX, IndexY) + 0.5) * InMipInvSize;
			float2 ScaledUVs = UV * 2.0 - 1.0;
			float3 NormalizedUVs = normalize(float3(ScaledUVs, 1.0));

			float SquaredUVs = 1 + dot(ScaledUVs, ScaledUVs);
			float TexelWeight = 4 / (sqrt(SquaredUVs) * SquaredUVs);

			// Positive/negative X faces.  Note that only one vector of an opposing pair can be in the positive half space of the normal,
			// so we select between the two faces, and adjust the UVs.  This cuts the number of texture samples in half.
			float NormalWeight = dot(Normal, float3(NormalizedUVs.z, -NormalizedUVs.y, -NormalizedUVs.x));
			int3 FaceUV = NormalWeight >= 0.0 ? int3(IndexX, IndexY, 0) : int3(IndexX, InMipSizeMinusOne - IndexY, 1);

			NormalWeight = TexelWeight * abs(NormalWeight);
			Irradiance += InCubeFaces.Load(int4(FaceUV, 0)).rgb * NormalWeight;
			TotalWeight += NormalWeight;

			// Positive/negative Y faces.
			NormalWeight = dot(Normal, float3(NormalizedUVs.x, NormalizedUVs.z, NormalizedUVs.y));
			FaceUV = NormalWeight >= 0.0 ? int3(IndexX, IndexY, 2) : int3(InMipSizeMinusOne - IndexX, IndexY, 3);

			NormalWeight = TexelWeight * abs(NormalWeight);
			Irradiance += InCubeFaces.Load(int4(FaceUV, 0)).rgb * NormalWeight;
			TotalWeight += NormalWeight;
				
			// Positive/negative Z faces.			
			NormalWeight = dot(Normal, float3(NormalizedUVs.x, -NormalizedUVs.y, NormalizedUVs.z));
			FaceUV = NormalWeight >= 0.0 ? int3(IndexX, IndexY, 4) : int3(IndexX, InMipSizeMinusOne - IndexY, 5);

			NormalWeight = TexelWeight * abs(NormalWeight);
			Irradiance += InCubeFaces.Load(int4(FaceUV, 0)).rgb * NormalWeight;
			TotalWeight += NormalWeight;
		}
	}

	OutCubeFaces[DispatchThreadId] = float4(Irradiance * (1.0 / TotalWeight), 1.0);
}


#ifdef SHADER_DIFFUSE_TO_SH

#include "SHCommon.ush"
#include "MonteCarlo.ush"

RWStructuredBuffer<float4> OutIrradianceEnvMapSH;
float UniformSampleSolidAngle;

#define THREADGROUP_SIZE (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)
groupshared FThreeBandSHVectorRGB IrradianceSHShared[THREADGROUP_SIZE];

FThreeBandSHVectorRGB SampleSHRGB(in float3 SampleDirection, in float weight, in float MipLevel)
{
	const float3 SampleColor = SourceCubemapTexture.SampleLevel(SourceCubemapSampler, SampleDirection, MipLevel).rgb;

	FThreeBandSHVector Sh3Vector = SHBasisFunction3(SampleDirection);
	FThreeBandSHVectorRGB Result;
	Result.R = MulSH3(Sh3Vector, SampleColor.r * weight);
	Result.G = MulSH3(Sh3Vector, SampleColor.g * weight);
	Result.B = MulSH3(Sh3Vector, SampleColor.b * weight);
	return Result;
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void ComputeSkyEnvMapDiffuseIrradianceCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const uint LinearIndex = THREADGROUP_SIZE_X * ThreadId.y + ThreadId.x;

#if 1

	// For a 128x128 cubemap, sampling mip level 2 with only 8x8 samples matches closely the super sampled version.
	const float3 SampleDirection = UniformSampleSphere((float2(ThreadId.xy)+0.5f) / float2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y)).xyz;
	IrradianceSHShared[LinearIndex] = SampleSHRGB(SampleDirection, UniformSampleSolidAngle, MipIndex);

#else

	FThreeBandSHVectorRGB IrradianceSHAcc;
	IrradianceSHAcc.R.V0 = IrradianceSHAcc.R.V1 = 0.0f; IrradianceSHAcc.R.V2 = 0.0f;
	IrradianceSHAcc.G.V0 = IrradianceSHAcc.G.V1 = 0.0f; IrradianceSHAcc.G.V2 = 0.0f;
	IrradianceSHAcc.B.V0 = IrradianceSHAcc.B.V1 = 0.0f; IrradianceSHAcc.B.V2 = 0.0f;

	// Uniform super sampling.
	// For a 128x128 cubemap, sampling mip level 0 with only 4x4 samples matches closely reference.
	const uint MipLevel = 0;
	const float SuperSampleAxisCount = 4.0f;
	const float SuperSampleCount = SuperSampleAxisCount * SuperSampleAxisCount;
	const float SuperSampleUniformSampleSolidAngle = UniformSampleSolidAngle / SuperSampleCount;
	for (float U = 0.0f; U < 1.0f; U += 1.0f / SuperSampleAxisCount)
	{
		for (float V = 0.0f; V < 1.0f; V += 1.0f / SuperSampleAxisCount)
		{
			const float3 SampleDirection = UniformSampleSphere((float2(ThreadId.xy) + float2(U,V)) / float2(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y)).xyz;
			IrradianceSHAcc = AddSH3(IrradianceSHAcc, SampleSHRGB(SampleDirection, SuperSampleUniformSampleSolidAngle, MipLevel));
		}
	}

	IrradianceSHShared[LinearIndex] = IrradianceSHAcc;

#endif

#if THREADGROUP_SIZE != 64
#error That is the only reduction supported today
#endif

	// Wait for all group threads to be done
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 32)
	{
		IrradianceSHShared[LinearIndex] = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 32]);
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 16)
	{
		IrradianceSHShared[LinearIndex] = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 16]);
	}
	GroupMemoryBarrierWithGroupSync();

	// The smallest wave size is 16 on Intel hardware. So now we can do simple math operations without group sync.

	if (LinearIndex < 8)
	{
		IrradianceSHShared[LinearIndex] = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 8]);
	}
	if (LinearIndex < 4)
	{
		IrradianceSHShared[LinearIndex] = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 4]);
	}
	if (LinearIndex < 2)
	{
		IrradianceSHShared[LinearIndex] = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 2]);
	}

	if (LinearIndex < 1)
	{
		FThreeBandSHVectorRGB SkyIrradiance = AddSH3(IrradianceSHShared[LinearIndex], IrradianceSHShared[LinearIndex + 1]);

		// Pack the SH coefficients in a way that makes applying the lighting use the least shader instructions
		// This has the diffuse convolution coefficients baked in. See "Stupid Spherical Harmonics (SH) Tricks".
		// Also see UpdateSkyIrradianceGpuBuffer.
		const float SqrtPI = sqrt(PI);
		const float Coefficient0 = 1.0f / (2.0f * SqrtPI);
		const float Coefficient1 = sqrt(3.0f) / (3.0f * SqrtPI);
		const float Coefficient2 = sqrt(15.0f) / (8.0f * SqrtPI);
		const float Coefficient3 = sqrt(5.0f) / (16.0f * SqrtPI);
		const float Coefficient4 = 0.5f * Coefficient2;

		OutIrradianceEnvMapSH[0].x = -Coefficient1 * SkyIrradiance.R.V0[3];
		OutIrradianceEnvMapSH[0].y = -Coefficient1 * SkyIrradiance.R.V0[1];
		OutIrradianceEnvMapSH[0].z =  Coefficient1 * SkyIrradiance.R.V0[2];
		OutIrradianceEnvMapSH[0].w =  Coefficient0 * SkyIrradiance.R.V0[0] - Coefficient3 * SkyIrradiance.R.V1[2];//[6];

		OutIrradianceEnvMapSH[1].x = -Coefficient1 * SkyIrradiance.G.V0[3];
		OutIrradianceEnvMapSH[1].y = -Coefficient1 * SkyIrradiance.G.V0[1];
		OutIrradianceEnvMapSH[1].z =  Coefficient1 * SkyIrradiance.G.V0[2];
		OutIrradianceEnvMapSH[1].w =  Coefficient0 * SkyIrradiance.G.V0[0] - Coefficient3 * SkyIrradiance.G.V1[2];//[6];

		OutIrradianceEnvMapSH[2].x = -Coefficient1 * SkyIrradiance.B.V0[3];
		OutIrradianceEnvMapSH[2].y = -Coefficient1 * SkyIrradiance.B.V0[1];
		OutIrradianceEnvMapSH[2].z =  Coefficient1 * SkyIrradiance.B.V0[2];
		OutIrradianceEnvMapSH[2].w =  Coefficient0 * SkyIrradiance.B.V0[0] - Coefficient3 * SkyIrradiance.B.V1[2];//[6];

		OutIrradianceEnvMapSH[3].x =  Coefficient2 * SkyIrradiance.R.V1[0];//[4];
		OutIrradianceEnvMapSH[3].y = -Coefficient2 * SkyIrradiance.R.V1[1];//V[5];
		OutIrradianceEnvMapSH[3].z = 3.0f * Coefficient3 * SkyIrradiance.R.V1[2];//[6];
		OutIrradianceEnvMapSH[3].w = -Coefficient2 * SkyIrradiance.R.V1[3];//[7];

		OutIrradianceEnvMapSH[4].x =  Coefficient2 * SkyIrradiance.G.V1[0];//[4];
		OutIrradianceEnvMapSH[4].y = -Coefficient2 * SkyIrradiance.G.V1[1];//[5];
		OutIrradianceEnvMapSH[4].z = 3.0f * Coefficient3 * SkyIrradiance.G.V1[2];//[6];
		OutIrradianceEnvMapSH[4].w = -Coefficient2 * SkyIrradiance.G.V1[3];//[7];

		OutIrradianceEnvMapSH[5].x =  Coefficient2 * SkyIrradiance.B.V1[0];//[4];
		OutIrradianceEnvMapSH[5].y = -Coefficient2 * SkyIrradiance.B.V1[1];//[5];
		OutIrradianceEnvMapSH[5].z = 3.0f * Coefficient3 * SkyIrradiance.B.V1[2];//[6];
		OutIrradianceEnvMapSH[5].w = -Coefficient2 * SkyIrradiance.B.V1[3];//[7];

		OutIrradianceEnvMapSH[6].x = Coefficient4 * SkyIrradiance.R.V2;//[8];
		OutIrradianceEnvMapSH[6].y = Coefficient4 * SkyIrradiance.G.V2;//[8];
		OutIrradianceEnvMapSH[6].z = Coefficient4 * SkyIrradiance.B.V2;//[8];
		OutIrradianceEnvMapSH[6].w = 1.0f;

		// We inverse the weights applied onto the non directional SH component to recover the average sky irradiance.
		const float3 DummyVector = float3(0.0, 0.0, 1.0);
		const float SHBasisL0Weight = SHBasisFunction3(DummyVector).V0.x;
		const float InvSHBand0Weights = 1.0f / (SHBasisL0Weight * 4.0f * PI);
		const float SkyAverageIrradianceR = SkyIrradiance.R.V0.x * InvSHBand0Weights;
		const float SkyAverageIrradianceG = SkyIrradiance.G.V0.x * InvSHBand0Weights;
		const float SkyAverageIrradianceB = SkyIrradiance.B.V0.x * InvSHBand0Weights;
		const float SkyAverageIrradiance  = dot(float3(SkyAverageIrradianceR, SkyAverageIrradianceG, SkyAverageIrradianceB), .3333f);
		OutIrradianceEnvMapSH[7].x = SkyAverageIrradiance;
		OutIrradianceEnvMapSH[7].y = SkyAverageIrradiance;
		OutIrradianceEnvMapSH[7].z = SkyAverageIrradiance;
		OutIrradianceEnvMapSH[7].w = SkyAverageIrradiance;
	}
}
#endif



#ifdef REALTIME_REFLECTION_HEIGHT_FOG

#include "HeightFogCommon.ush"

float3 SkyLightPosition;

#if PERMUTATION_DEPTHTEXTURE
Texture2D DepthTexture;
#endif

void RenderRealTimeReflectionHeightFogVS(
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION,
	out float3 OutScreenVector : TEXCOORD0)
{
	float2 UV = -1.0f;
	UV = VertexId == 1 ? float2(-1.0f, 3.0f) : UV;
	UV = VertexId == 2 ? float2(3.0f, -1.0f) : UV;
	Position = float4(UV, 0.5f, 1.0f);
	OutScreenVector = ScreenVectorFromScreenRect(float4(Position.xy, 1, 0));
}


void RenderRealTimeReflectionHeightFogPS(
	in float4 SVPos : SV_POSITION,
	in float3 ScreenVector : TEXCOORD0,
	out float4 OutLuminance : SV_Target0
)
{
	ResolvedView = ResolveView();

#if PERMUTATION_DEPTHTEXTURE
	float DeviceZ = DepthTexture.Load(int3(SVPos.xy, 0)).x;
#else
	float DeviceZ = FarDepthValue;
#endif

	DeviceZ = max(1.0e-10, DeviceZ); // For ConvertFromDeviceZ to return a valid value (non inf).

	float SceneDepth = ConvertFromDeviceZ(DeviceZ);
	float3 WorldPositionRelativeToCamera = ScreenVector.xyz * SceneDepth;

	// Evaluate the height fog the actual sky light position to match the capture position.
	const float ExcludeDistance = 0;
	const float EyeIndex = 0;
	const bool bOverrideOrigin = true;
	const float3 OverrideRayStartRelativeToOrigin = SkyLightPosition;
	const float OverrideRayLength = length(WorldPositionRelativeToCamera);
	const float3 OverrideRayDir = WorldPositionRelativeToCamera / max(0.000001, OverrideRayLength);
	float4 HeightFogInscatteringAndOpacity = GetExponentialHeightFog(WorldPositionRelativeToCamera, ExcludeDistance, EyeIndex, PrimaryView, bOverrideOrigin, OverrideRayStartRelativeToOrigin, OverrideRayDir, OverrideRayLength);

	const float OutputPreExposure = (ResolvedView.RealTimeReflectionCapture ? ResolvedView.RealTimeReflectionCapturePreExposure : ResolvedView.PreExposure);
	HeightFogInscatteringAndOpacity.rgb *= OutputPreExposure;

	OutLuminance = HeightFogInscatteringAndOpacity;
}

#endif

#ifdef VISUALIZE_ILLUMINANCE_METER

#include "/Engine/Private/ShaderPrint.ush"
#include "/Engine/Private/SkyAtmosphereCommon.ush"

int CubeMipIndexToSampleIlluminance;
float3 SkyLightCaptureWorlPos;

int CurrentCubemapResolution;
int SkyLightCubemapMaxResolution;
int HardwareCubemapMaxResolution;

TextureCube SkyLightCubeTexture;
SamplerState SkyLightCubeSampler;

float3 GetIlluminance()
{
	float4 IlluminanceAcc = 0;
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3(-1, 0, 0), CubeMipIndexToSampleIlluminance);
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3( 1, 0, 0), CubeMipIndexToSampleIlluminance);
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3( 0,-1, 0), CubeMipIndexToSampleIlluminance);
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3( 0, 1, 0), CubeMipIndexToSampleIlluminance);
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3( 0, 0,-1), CubeMipIndexToSampleIlluminance);
	IlluminanceAcc += TextureCubeSampleLevel(SourceCubemapTexture, SourceCubemapSampler, float3( 0, 0, 1), CubeMipIndexToSampleIlluminance);

	// See https://www.ppsloan.org/publications/StupidSH36.pdf, p.9.
	IlluminanceAcc.rgb *= 4.0 * PI / IlluminanceAcc.a;

	return IlluminanceAcc.rgb;
}

float4 SampleCubeAsLatLon(TextureCube CubeTexture, SamplerState CubeSampler, float2 UV)
{
	float HAngle = UV.x * 2.0 * PI;
	float VAngle = UV.y * PI - PI / 2;
	float CosVAngle = cos(VAngle);
	float3 DirFromLatLong = float3(cos(HAngle) * CosVAngle, sin(HAngle) * CosVAngle, -sin(VAngle));

	float4 DebugVis = TextureCubeSampleLevel(CubeTexture, CubeSampler, DirFromLatLong, 0);
	return DebugVis;
}

float4 SampleCubeAsSphere(TextureCube CubeTexture, SamplerState CubeSampler, float2 UV)
{
	float2 Dir2D = (UV - 0.5) * 2.0;

	float4 DebugVis = float4(0.0, 0.0, 0.0, 1.0);
	if (length(Dir2D) <= 1.0)
	{
		float3 N = 0;
		const float3 RayO = float3(UV, -1);
		const float3 RayD = float3(0.0f, 0.0f, 1.0f);
		const float4 Sphere = float4(0.5f, 0.5f, 0.f, 0.5);
		const float2 Hit = RayIntersectSphere(RayO, RayD, Sphere);
		if (Hit.x >= 0)
		{
			const float3 P = RayO + RayD * Hit.x;
			N = normalize(P - Sphere.xyz);
			N *= float3(1.0, -1.0, 1.0);
		}

		DebugVis = TextureCubeSampleLevel(CubeTexture, CubeSampler, mul(N, (float3x3)ResolvedView.ViewToTranslatedWorld), 0);
	}

	return DebugVis;
}

void VisualizeIlluminanceMeterPS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	ResolvedView = ResolveView();

	const uint2 PixelPosDynRes = uint2(float2(SVPos.xy) * ResolvedView.ViewResolutionFraction);
	float2 BufferUV = SvPositionToBufferUV(float4(PixelPosDynRes, SVPos.zw));

	OutColor = 0.0f;

	const uint2 IlluMapPreviewOrigin	= uint2(100, 110);
	const uint2 IlluMapPreviewSize		= uint2(500, 250);

	const uint2 SkyMapPreviewOrigin		= uint2(100, 370);
	const uint2 SkyMapPreviewSize		= uint2(500, 250);	

	const uint2 SphereIlluMapPreviewOrigin	= IlluMapPreviewOrigin + uint2(IlluMapPreviewSize.x + 10, 0);
	const uint2 SphereIlluMapPreviewSize	= uint2(250, 250);

	const uint2 SphereSkyMapPreviewOrigin	= SkyMapPreviewOrigin + uint2(SkyMapPreviewSize.x + 10, 0);
	const uint2 SphereSkyMapPreviewSize		= uint2(250, 250);

	if (all(uint2(SVPos.xy) == uint2(1, 1)))
	{
		float3 SkyLightIlluminance = GetIlluminance();

#if PROJECT_SUPPORT_SKY_ATMOSPHERE
		const float3 PlanetCenterToTranslatedWorldPos = (DFFastAddDemote(SkyLightCaptureWorlPos, ResolvedView.PreViewTranslation) - ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;
		// GetAtmosphereTransmittance does a shadow test against the virtual planet.
		const float3 TransmittanceToLight = GetAtmosphereTransmittance(
			PlanetCenterToTranslatedWorldPos, ResolvedView.AtmosphereLightDirection[0].xyz, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
			View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);

		const float3 SunLightIlluminance = ResolvedView.AtmosphereLightIlluminanceOuterSpace[0].rgb * TransmittanceToLight;
#else
		const float3 SunLightIlluminance = ResolvedView.AtmosphereLightIlluminanceOuterSpace[0].rgb;
#endif

		float SkyIlluminance = max3(SkyLightIlluminance.r, SkyLightIlluminance.g, SkyLightIlluminance.b);
		float SunIlluminance = max3(SunLightIlluminance.r, SunLightIlluminance.g, SunLightIlluminance.b);
		//float SkyIlluminance = dot(SkyLightIlluminance.rgb, (1.0/3.0).xxx);
		//float SunIlluminance = dot(SunLightIlluminance.rgb, (1.0/3.0).xxx);
		float TotalIlluminance = SkyIlluminance + SunIlluminance;

		FFontColor DefaultFontColor = InitFontColor(0.9, 0.9, 0.9);
		FShaderPrintContext Context = InitShaderPrintContext(true, uint2(110, 20));
		Print(Context, TEXT("Current sky light resolution = "), DefaultFontColor);
		Print(Context, CurrentCubemapResolution, DefaultFontColor);
		Print(Context, TEXT("(Hardware max resolution = "), DefaultFontColor);
		Print(Context, HardwareCubemapMaxResolution, DefaultFontColor);
		Print(Context, TEXT(", r.SkyLight.CubemapMaxResolution = "), DefaultFontColor);
		Print(Context, SkyLightCubemapMaxResolution, DefaultFontColor);
		Print(Context, TEXT(")"), DefaultFontColor);

		Context = InitShaderPrintContext(true, uint2(110, 70));
		Print(Context, TEXT("Sky light illuminance = "), DefaultFontColor);
		Print(Context, SkyLightIlluminance.r, InitFontColor(1.0, 0.8, 0.8));
		Print(Context, SkyLightIlluminance.g, InitFontColor(0.8, 1.0, 0.8));
		Print(Context, SkyLightIlluminance.b, InitFontColor(0.8, 0.8, 1.0));
		Print(Context, 100.0 * SkyIlluminance / TotalIlluminance, InitFontColor(0.7, 0.7, 0.7));
		Print(Context, TEXT("%"), InitFontColor(0.7, 0.7, 0.7));

		Newline(Context);

		Print(Context, TEXT("Sun light illuminance = "), DefaultFontColor);
		Print(Context, SunLightIlluminance.r, InitFontColor(1.0, 0.8, 0.8));
		Print(Context, SunLightIlluminance.g, InitFontColor(0.8, 1.0, 0.8));
		Print(Context, SunLightIlluminance.b, InitFontColor(0.8, 0.8, 1.0));
		Print(Context, 100.0 * SunIlluminance / TotalIlluminance, InitFontColor(0.7, 0.7, 0.7));
		Print(Context, TEXT("%"), InitFontColor(0.7, 0.7, 0.7));

		Newline(Context);

		Print(Context, TEXT("Total illuminance     = "), DefaultFontColor);
		Print(Context, SkyLightIlluminance.r + SunLightIlluminance.r, InitFontColor(1.0, 0.8, 0.8));
		Print(Context, SkyLightIlluminance.g + SunLightIlluminance.g, InitFontColor(0.8, 1.0, 0.8));
		Print(Context, SkyLightIlluminance.b + SunLightIlluminance.b, InitFontColor(0.8, 0.8, 1.0));
		Print(Context, 100.0 * (SkyIlluminance + SunIlluminance) / TotalIlluminance, InitFontColor(0.7, 0.7, 0.7));
		Print(Context, TEXT("%"), InitFontColor(0.7, 0.7, 0.7));

		Context = InitShaderPrintContext(true, SphereIlluMapPreviewOrigin + uint2(SphereIlluMapPreviewSize.x + 10, SphereIlluMapPreviewSize.y / 2));
		Print(Context, TEXT("Illuminance meter view"), InitFontColor(0.9, 0.9, 0.9));

		Context = InitShaderPrintContext(true, SphereSkyMapPreviewOrigin + uint2(SphereSkyMapPreviewSize.x + 10, SphereSkyMapPreviewSize.y / 2));
		Print(Context, TEXT("Sky light cube map view"), InitFontColor(0.9, 0.9, 0.9));

	}
	else if (all(uint2(SVPos.xy) >= IlluMapPreviewOrigin) && all(uint2(SVPos.xy) <= (IlluMapPreviewOrigin + IlluMapPreviewSize)))
	{
		float2 UV = (SVPos.xy - float2(IlluMapPreviewOrigin)) / float2(IlluMapPreviewSize);
		float4 DebugVis = SampleCubeAsLatLon(SourceCubemapTexture, SourceCubemapSampler, UV);
		DebugVis.rgb *= 1.0 / DebugVis.a;	// This is a way to remove the texel dArea factor to show what luminance is integrated as a unction of NoL.
		OutColor = float4(DebugVis.rgb * ResolvedView.PreExposure, 0.0);
	}
	else if (all(uint2(SVPos.xy) >= SkyMapPreviewOrigin) && all(uint2(SVPos.xy) <= (SkyMapPreviewOrigin + SkyMapPreviewSize)))
	{
		float2 UV = (SVPos.xy - float2(SkyMapPreviewOrigin)) / float2(SkyMapPreviewSize);
		float4 DebugVis = SampleCubeAsLatLon(SkyLightCubeTexture, SkyLightCubeSampler, UV);
		DebugVis.rgb *= ResolvedView.SkyLightColor.rgb;
		OutColor = float4(DebugVis.rgb * ResolvedView.PreExposure, 0.0);
	}
	else if (all(uint2(SVPos.xy) >= SphereIlluMapPreviewOrigin) && all(uint2(SVPos.xy) <= (SphereIlluMapPreviewOrigin + SphereIlluMapPreviewSize)))
	{
		float2 UV = (SVPos.xy - float2(SphereIlluMapPreviewOrigin)) / float2(SphereIlluMapPreviewSize);
		float4 DebugVis = SampleCubeAsSphere(SourceCubemapTexture, SourceCubemapSampler, UV);
		DebugVis.rgb *= 1.0 / DebugVis.a;	// This is a way to remove the texel dArea factor to show what luminance is integrated as a unction of NoL.
		OutColor = float4(DebugVis.rgb * ResolvedView.PreExposure, 0.0);
	}
	else if (all(uint2(SVPos.xy) >= SphereSkyMapPreviewOrigin) && all(uint2(SVPos.xy) <= (SphereSkyMapPreviewOrigin + SphereSkyMapPreviewSize)))
	{
		float2 UV = (SVPos.xy - float2(SphereSkyMapPreviewOrigin)) / float2(SphereSkyMapPreviewSize);
		float4 DebugVis = SampleCubeAsSphere(SkyLightCubeTexture, SkyLightCubeSampler, UV);
		DebugVis.rgb *= ResolvedView.SkyLightColor.rgb;
		OutColor = float4(DebugVis.rgb * ResolvedView.PreExposure, 0.0);
	}
	else
	{
		clip(-1.0);
	}

	// TODO print sky as a sphere
}

#endif // DEBUG_ILLUMINANCE_METER

