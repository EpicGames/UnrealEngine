// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if (FEATURE_LEVEL <= FEATURE_LEVEL_ES3_1)
#define SkyIrradianceEnvironmentMap		View.MobileSkyIrradianceEnvironmentMap
#else
#define SkyIrradianceEnvironmentMap		View.SkyIrradianceEnvironmentMap
#endif

float GetSkyLightCubemapBrightness()
{
	return SkyIrradianceEnvironmentMap[7].x; // Refer to FSceneRenderer::UpdateSkyIrradianceGpuBuffer for more details.
}

#define REFLECTION_CAPTURE_ROUGHEST_MIP 1
#define REFLECTION_CAPTURE_ROUGHNESS_MIP_SCALE 1.2

// This sampler cannot be shared because they come from the texture assets setup by users. We could enforce those to be min/mag/mip trilinear filtering.
#define SharedSkyLightCubemapSampler					ReflectionStruct.SkyLightCubemapSampler
#define SharedSkyLightBlendDestinationCubemapSampler	ReflectionStruct.SkyLightBlendDestinationCubemapSampler

/** 
 * Compute absolute mip for a reflection capture cubemap given a roughness.
 */
half ComputeReflectionCaptureMipFromRoughness(half Roughness, half CubemapMaxMip)
{
	// Heuristic that maps roughness to mip level
	// This is done in a way such that a certain mip level will always have the same roughness, regardless of how many mips are in the texture
	// Using more mips in the cubemap just allows sharper reflections to be supported
	half LevelFrom1x1 = REFLECTION_CAPTURE_ROUGHEST_MIP - REFLECTION_CAPTURE_ROUGHNESS_MIP_SCALE * log2(max(Roughness, 0.001));
	return CubemapMaxMip - 1 - LevelFrom1x1;
}

float ComputeReflectionCaptureRoughnessFromMip(float Mip, half CubemapMaxMip)
{
	float LevelFrom1x1 = CubemapMaxMip - 1 - Mip;
	return exp2( ( REFLECTION_CAPTURE_ROUGHEST_MIP - LevelFrom1x1 ) / REFLECTION_CAPTURE_ROUGHNESS_MIP_SCALE );
}

// ReflectionStruct.SkyLightParameters: X = max mip, Y = 1 if sky light should be rendered, 0 otherwise, Z = 1 if sky light is dynamic, 0 otherwise, W = blend fraction. 

float3 GetSkyLightReflection(float3 ReflectionVector, float Roughness, out float OutSkyAverageBrightness)
{
	float AbsoluteSpecularMip = ComputeReflectionCaptureMipFromRoughness(Roughness, ReflectionStruct.SkyLightParameters.x);
	float3 Reflection = TextureCubeSampleLevel(ReflectionStruct.SkyLightCubemap, SharedSkyLightCubemapSampler, ReflectionVector, AbsoluteSpecularMip).rgb;

	OutSkyAverageBrightness = GetSkyLightCubemapBrightness() * Luminance(View.SkyLightColor.rgb);
	return Reflection * View.SkyLightColor.rgb;
}

float3 GetSkyLightReflectionSupportingBlend(float3 ReflectionVector, float Roughness, out float OutSkyAverageBrightness)
{
	float3 Reflection = GetSkyLightReflection(ReflectionVector, Roughness, OutSkyAverageBrightness);

	BRANCH
	if (ReflectionStruct.SkyLightParameters.w > 0)
	{
		float AbsoluteSpecularMip = ComputeReflectionCaptureMipFromRoughness(Roughness, ReflectionStruct.SkyLightParameters.x);
		float3 BlendDestinationReflection = TextureCubeSampleLevel(ReflectionStruct.SkyLightBlendDestinationCubemap, SharedSkyLightBlendDestinationCubemapSampler, ReflectionVector, AbsoluteSpecularMip).rgb;

		Reflection = lerp(Reflection, BlendDestinationReflection * View.SkyLightColor.rgb, ReflectionStruct.SkyLightParameters.w);
	}

	return Reflection;
}

bool ShouldSkyLightApplyPrecomputedBentNormalShadowing() {
	return View.SkyLightApplyPrecomputedBentNormalShadowingFlag != 0.0f;
}

bool ShouldSkyLightAffectReflection() {
	return View.SkyLightAffectReflectionFlag != 0.0f;
}

bool ShouldSkyLightAffectGlobalIllumination() {
	return View.SkyLightAffectGlobalIlluminationFlag != 0.0f;
}

/** 
 * Computes sky diffuse lighting from the SH irradiance map.  
 * This has the SH basis evaluation and diffuse convolution weights combined for minimal ALU's - see "Stupid Spherical Harmonics (SH) Tricks" 
 */
float3 GetSkySHDiffuse(float3 Normal)
{
	float4 NormalVector = float4(Normal, 1.0f); 

	float3 Intermediate0, Intermediate1, Intermediate2;
	Intermediate0.x = dot(SkyIrradianceEnvironmentMap[0], NormalVector);
	Intermediate0.y = dot(SkyIrradianceEnvironmentMap[1], NormalVector);
	Intermediate0.z = dot(SkyIrradianceEnvironmentMap[2], NormalVector);

	float4 vB = NormalVector.xyzz * NormalVector.yzzx;
	Intermediate1.x = dot(SkyIrradianceEnvironmentMap[3], vB);
	Intermediate1.y = dot(SkyIrradianceEnvironmentMap[4], vB);
	Intermediate1.z = dot(SkyIrradianceEnvironmentMap[5], vB);

	float vC = NormalVector.x * NormalVector.x - NormalVector.y * NormalVector.y;
	Intermediate2 = SkyIrradianceEnvironmentMap[6].xyz * vC;

	// max to not get negative colors
	return max(0, Intermediate0 + Intermediate1 + Intermediate2);
}

/**
* Computes sky diffuse lighting from the SH irradiance map.
* This has the SH basis evaluation and diffuse convolution weights combined for minimal ALU's - see "Stupid Spherical Harmonics (SH) Tricks"
* Only does the first 3 components for speed.
*/
float3 GetSkySHDiffuseSimple(float3 Normal)
{
	float4 NormalVector = float4(Normal, 1);

	float3 Intermediate0;
	Intermediate0.x = dot(SkyIrradianceEnvironmentMap[0], NormalVector);
	Intermediate0.y = dot(SkyIrradianceEnvironmentMap[1], NormalVector);
	Intermediate0.z = dot(SkyIrradianceEnvironmentMap[2], NormalVector);		

	// max to not get negative colors
	return max(0, Intermediate0);
}


// Point lobe in off-specular peak direction
half3 GetOffSpecularPeakReflectionDir(half3 Normal, half3 ReflectionVector, half Roughness)
{
	half a = Roughness * Roughness;
	return lerp( Normal, ReflectionVector, (1 - a) * ( sqrt(1 - a) + a ) );	
}

half GetSpecularOcclusion(half NoV, half RoughnessSq, half AO)
{
	return saturate( pow( NoV + AO, RoughnessSq ) - 1 + AO );
}

float3 GetLookupVectorForBoxCapture(float3 ReflectionVector, float3 WorldPosition, float4 BoxCapturePositionAndRadius, float4x4 RelativeWorldToBox, float4 BoxScales, float3 LocalCaptureOffset, float NormalAdjust, out float DistanceAlpha)
{
	// Transform the ray into the local space of the box, where it is an AABB with mins at -1 and maxs at 1
	float3 LocalRayStart		= mul(float4(WorldPosition - BoxCapturePositionAndRadius.xyz, 1), RelativeWorldToBox).xyz;
	float3 LocalRayDirection	= mul(float4(ReflectionVector,  0), RelativeWorldToBox).xyz;

	float3 InvRayDir = rcp(LocalRayDirection);
	
	//find the ray intersection with each of the 3 planes defined by the minimum extrema.
	float3 FirstPlaneIntersections = InvRayDir * (-1.0 - LocalRayStart);
	//find the ray intersection with each of the 3 planes defined by the maximum extrema.
	float3 SecondPlaneIntersections = InvRayDir * (1.0 - LocalRayStart);
	//get the furthest of these intersections along the ray
	float3 FurthestPlaneIntersections = max(FirstPlaneIntersections, SecondPlaneIntersections);

	//clamp the intersections to be between RayOrigin and RayEnd on the ray
	float Intersection = min(FurthestPlaneIntersections.x, min(FurthestPlaneIntersections.y, FurthestPlaneIntersections.z));

	// Compute the reprojected vector
	float3 IntersectPosition = WorldPosition + Intersection * ReflectionVector;
	float3 ProjectedCaptureVector = IntersectPosition - (BoxCapturePositionAndRadius.xyz + LocalCaptureOffset);

	// Compute the distance from the receiving pixel to the box for masking
	// Apply local to world scale to take scale into account without transforming back to world space
	// Shrink the box by the transition distance (BoxScales.w) so that the fade happens inside the box influence area	
	float BoxDistance = ComputeDistanceFromBoxToPoint(-(BoxScales.xyz - .5f * (BoxScales.w + NormalAdjust)), BoxScales.xyz - .5f * (BoxScales.w + NormalAdjust), LocalRayStart * BoxScales.xyz);

	// Setup a fade based on receiver distance to the box, hides the box influence shape
	DistanceAlpha = 1.0 - smoothstep(0, .7f * BoxScales.w, BoxDistance);

	return ProjectedCaptureVector;
}

// Version of function above for diffuse irradiance direction calculation.  Includes an intersection distance clamp to solve artifacts
// near the box boundary, and leaves out unused DistanceAlpha computation.
float3 GetLookupVectorForBoxCaptureDiffuse(float3 NormalVector, float3 WorldPosition, float4 BoxCapturePositionAndRadius, float4x4 RelativeWorldToBox, float4 BoxScales, float3 LocalCaptureOffset)
{
	// Transform the ray into the local space of the box, where it is an AABB with mins at -1 and maxs at 1
	float3 LocalRayStart = mul(float4(WorldPosition - BoxCapturePositionAndRadius.xyz, 1), RelativeWorldToBox).xyz;
	float3 LocalRayDirection = mul(float4(NormalVector, 0), RelativeWorldToBox).xyz;

	float3 InvRayDir = rcp(LocalRayDirection);
	
	//find the ray intersection with each of the 3 planes defined by the minimum extrema.
	float3 FirstPlaneIntersections = InvRayDir * (-1.0 - LocalRayStart);
	//find the ray intersection with each of the 3 planes defined by the maximum extrema.
	float3 SecondPlaneIntersections = InvRayDir * (1.0 - LocalRayStart);
	//get the furthest of these intersections along the ray
	float3 FurthestPlaneIntersections = max(FirstPlaneIntersections, SecondPlaneIntersections);

	//clamp the intersections to be between RayOrigin and RayEnd on the ray
	float Intersection = min(FurthestPlaneIntersections.x, min(FurthestPlaneIntersections.y, FurthestPlaneIntersections.z));

	// Apply an arbitrary minimum to the intersection distance, to avoid artifacts for points near the boundary (with short
	// intersection distances).  This effectively biases ProjectedCaptureVector towards "NormalVector" in such cases, at a
	// minimum additional ALU cost.  An "add" was also tried, but this produced better visual results.
	Intersection = max(Intersection, 1000.0);
	
	// Compute the reprojected vector
	float3 IntersectPosition = WorldPosition + Intersection * NormalVector;
	float3 ProjectedCaptureVector = IntersectPosition - (BoxCapturePositionAndRadius.xyz + LocalCaptureOffset);

	return ProjectedCaptureVector;
}

float3 GetLookupVectorForSphereCapture(float3 ReflectionVector, float3 WorldPosition, float4 SphereCapturePositionAndRadius, float NormalizedDistanceToCapture, float3 LocalCaptureOffset, inout float DistanceAlpha)
{
	float3 ProjectedCaptureVector = ReflectionVector;
	float ProjectionSphereRadius = SphereCapturePositionAndRadius.w;
	float SphereRadiusSquared = ProjectionSphereRadius * ProjectionSphereRadius;

	float3 LocalPosition = WorldPosition - SphereCapturePositionAndRadius.xyz;
	float LocalPositionSqr = dot(LocalPosition, LocalPosition);

	// Find the intersection between the ray along the reflection vector and the capture's sphere
	float3 QuadraticCoef;
	QuadraticCoef.x = 1;
	QuadraticCoef.y = dot(ReflectionVector, LocalPosition);
	QuadraticCoef.z = LocalPositionSqr - SphereRadiusSquared;

	float Determinant = QuadraticCoef.y * QuadraticCoef.y - QuadraticCoef.z;

	// Only continue if the ray intersects the sphere
	FLATTEN
	if (Determinant >= 0)
	{
		float FarIntersection = sqrt(Determinant) - QuadraticCoef.y;

		float3 LocalIntersectionPosition = LocalPosition + FarIntersection * ReflectionVector;
		ProjectedCaptureVector = LocalIntersectionPosition - LocalCaptureOffset;
		// Note: some compilers don't handle smoothstep min > max (this was 1, .6)
		//DistanceAlpha = 1.0 - smoothstep(.6, 1, NormalizedDistanceToCapture);

		float x = saturate( 2.5 * NormalizedDistanceToCapture - 1.5 );
		DistanceAlpha = 1 - x*x*(3 - 2*x);
	}
	return ProjectedCaptureVector;
}

half ComputeMixingWeight(half IndirectIrradiance, half AverageBrightness, half Roughness)
{
	// Mirror surfaces should have no mixing, so they match reflections from other sources (SSR, planar reflections)
	half MixingAlpha = smoothstep(0, 1, saturate(Roughness * View.ReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight.x + View.ReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight.y));

	// We have high frequency directional data but low frequency spatial data in the envmap.
	// We have high frequency spatial data but low frequency directional data in the lightmap.
	// So, we combine the two for the best of both. This is done by removing the low spatial frequencies from the envmap and replacing them with the lightmap data.
	// This is only done with luma so as to not get odd color shifting.
	half MixingWeight = IndirectIrradiance / max(AverageBrightness, .0001f);

	MixingWeight = min(MixingWeight, View.ReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight.z);

	return lerp(1.0f, MixingWeight, MixingAlpha);
}

#if ES3_1_PROFILE
#define GetReflectionPositionAndRadius(CaptureIndex) ReflectionCaptureES31.PositionHighAndRadius[CaptureIndex]
#define GetReflectionCaptureProperties(CaptureIndex) ReflectionCaptureES31.CaptureProperties[CaptureIndex]
#define GetReflectionCaptureOffsetAndAverageBrightness(CaptureIndex) ReflectionCaptureES31.CaptureOffsetAndAverageBrightness[CaptureIndex]
#define GetReflectionBoxTransform(CaptureIndex) ReflectionCaptureES31.BoxTransform[CaptureIndex]
#define GetReflectionBoxScales(CaptureIndex) ReflectionCaptureES31.BoxScales[CaptureIndex]
#define GetReflectionPositionLow(CaptureIndex) ReflectionCaptureES31.PositionLow[CaptureIndex]
#else
#define GetReflectionPositionAndRadius(CaptureIndex) ReflectionCaptureSM5.PositionHighAndRadius[CaptureIndex]
#define GetReflectionCaptureProperties(CaptureIndex) ReflectionCaptureSM5.CaptureProperties[CaptureIndex]
#define GetReflectionCaptureOffsetAndAverageBrightness(CaptureIndex) ReflectionCaptureSM5.CaptureOffsetAndAverageBrightness[CaptureIndex]
#define GetReflectionBoxTransform(CaptureIndex) ReflectionCaptureSM5.BoxTransform[CaptureIndex]
#define GetReflectionBoxScales(CaptureIndex) ReflectionCaptureSM5.BoxScales[CaptureIndex]
#define GetReflectionPositionLow(CaptureIndex) ReflectionCaptureSM5.PositionLow[CaptureIndex]
#endif