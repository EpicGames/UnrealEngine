// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RenderGraphUtilities.usf
=============================================================================*/

#include "Common.ush"

Buffer<uint4> RectCoordBuffer; // [MinX, MinY, MaxX, MaxY]
Buffer<uint4> RectUVBuffer;

float DownsampleFactor;
float2 InvViewSize;
float2 InvTextureSize;
float DepthIncrement;

bool2 VertMax(uint VertexId)
{
	bool2 bVertMax;
	bVertMax.x = VertexId == 1 || VertexId == 2 || VertexId == 4;
	bVertMax.y = VertexId == 2 || VertexId == 4 || VertexId == 5;
	return bVertMax;
}

void RasterizeToRectsVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV : TEXCOORD0,
	out float2 OutRectUV : TEXCOORD1,
	out float OutRectIndex : RECT_INDEX)
{
	uint4 RectCoord = RectCoordBuffer[InstanceId] * DownsampleFactor;
	uint2 VertexCoord = select(VertMax(VertexId), RectCoord.zw, RectCoord.xy);

	float4 RectUV = RectCoord * InvViewSize.xyxy;
	#if RECT_UV
	{
		RectUV = RectUVBuffer[InstanceId] * DownsampleFactor * InvTextureSize.xyxy;
	}
	#endif
	float2 VertexUV = select(VertMax(VertexId), RectUV.zw, RectUV.xy);

	float Z = InstanceId * DepthIncrement;

	OutPosition = float4(float2(VertexCoord) * InvViewSize * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), Z, 1.0f);
	OutUV = VertexUV;
	OutRectUV.x = VertMax(VertexId).x ? 1.0f : 0.0f;
	OutRectUV.y = VertMax(VertexId).y ? 1.0f : 0.0f;
	OutRectIndex = InstanceId;
}