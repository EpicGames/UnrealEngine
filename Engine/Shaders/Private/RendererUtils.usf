// Copyright Epic Games, Inc. All Rights Reserved.
#include "Common.ush"

RWBuffer<uint> OutIndirectArgsBuffer;
uint NumIndirectArgs;
uint IndirectArgStride;
int3 DimClearValue;

// Set dispatch Args to DimClearValue.XYZ (3D grid dim) and any other fields to 0 ready for atomic adding on GPU
[numthreads(64, 1, 1)]
void ClearIndirectDispatchArgsCS(uint IndirectArgIndex : SV_DispatchThreadID)
{
	if (IndirectArgIndex < NumIndirectArgs)
	{
		OutIndirectArgsBuffer[IndirectArgIndex * IndirectArgStride + 0] = DimClearValue.x; // X-group count
		OutIndirectArgsBuffer[IndirectArgIndex * IndirectArgStride + 1] = DimClearValue.y; // Y-group count
		OutIndirectArgsBuffer[IndirectArgIndex * IndirectArgStride + 2] = DimClearValue.z; // Z-group count

		// Clear remaining auxiliary or padding fields to zero
		for (int Index = 3; Index < IndirectArgStride; ++Index)
		{
			OutIndirectArgsBuffer[IndirectArgIndex * IndirectArgStride + Index] = 0;
		}
	}
}
