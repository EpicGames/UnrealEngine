// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ResolvePixelShader.usf: Resolve pixel shader source.
=============================================================================*/

#include "Common.ush"

#ifndef DEPTH_RESOLVE_TEXTUREARRAY
	#define DEPTH_RESOLVE_TEXTUREARRAY 0
#endif 

#if DEPTH_RESOLVE_TEXTUREARRAY
Texture2DMSArray<float4> UnresolvedSurface;
#else
Texture2DMS<float4> UnresolvedSurface;
#endif


void MainDepth(
	float2 InUV : TEXCOORD0,
#if DEPTH_RESOLVE_TEXTUREARRAY
	nointerpolation uint ViewId : VIEW_ID,
#endif
	out float OutDepth : SV_DEPTH
	)
{
#ifndef DEPTH_RESOLVE_NUM_SAMPLES
	#if DEPTH_RESOLVE_TEXTUREARRAY
		uint4 SurfaceDimensions;
		UnresolvedSurface.GetDimensions(SurfaceDimensions.x, SurfaceDimensions.y, SurfaceDimensions.z, SurfaceDimensions.w);
		int NumSurfaceSamples = (int)SurfaceDimensions.w;
	#else
		uint3 SurfaceDimensions;
		UnresolvedSurface.GetDimensions(SurfaceDimensions.x, SurfaceDimensions.y, SurfaceDimensions.z);
		int NumSurfaceSamples = (int)SurfaceDimensions.z;
	#endif
#else
	int NumSurfaceSamples = DEPTH_RESOLVE_NUM_SAMPLES;
#endif

#if DEPTH_RESOLVE_TEXTUREARRAY
	int3 IntUV = int3(trunc(InUV.x), trunc(InUV.y), ViewId);
#else
	int2 IntUV = trunc(InUV);
#endif
	float ResolvedDepth = UnresolvedSurface.Load(IntUV, 0).r;

	for (int SampleIndex = 1; SampleIndex < NumSurfaceSamples; ++SampleIndex)
	{
		float Sample = UnresolvedSurface.Load(IntUV, SampleIndex).r;

		// Note that max depth actually means closest depth, since 1/depth is stored. 
		ResolvedDepth = max(ResolvedDepth, Sample);
	}

	OutDepth = ResolvedDepth;
}

#if !DEPTH_RESOLVE_TEXTUREARRAY
uint SingleSampleIndex;

void MainSingleSample(
	float2 InUV : TEXCOORD0,
	nointerpolation uint ViewId : VIEW_ID,
	out float4 OutColor : SV_Target0
	)
{
	int2 IntUV = trunc(InUV);

	OutColor = UnresolvedSurface.Load(IntUV,SingleSampleIndex);
}
#endif // !DEPTH_RESOLVE_TEXTUREARRAY