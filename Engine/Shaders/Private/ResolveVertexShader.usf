// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ResolveVertexShader.usf: Resolve vertex shader source.
=============================================================================*/

#include "Common.ush"

#ifndef DEPTH_RESOLVE_TEXTUREARRAY
	#define DEPTH_RESOLVE_TEXTUREARRAY 0
#endif 

float4 PositionMinMax; // xy = min [x, y], zw = max [x, y]
float4 UVMinMax;       // xy = min [u, v], zw = max [u, v]

void Main(
	in uint InstanceID					: SV_InstanceID,
	uint VertexID						: SV_VertexID,
	out float2 OutUV					: TEXCOORD0,
#if DEPTH_RESOLVE_TEXTUREARRAY
	out nointerpolation uint OutViewId  : VIEW_ID,
	out uint OutRTIndex					: SV_RenderTargetArrayIndex,
#endif
	out float4 OutPosition				: SV_POSITION
	)
{
	// Generate these positions:
	// ViewId: 0
	// [0, 1], [1, 1], [0, 0], [1, 0]
	// ViewId: 1
	// [0, 1], [1, 1], [0, 0], [1, 0]

	float2 Pos = float2(
		float(VertexID & 1),
		float(1 - (VertexID >> 1))
	);

	OutPosition = float4(lerp(PositionMinMax.xy, PositionMinMax.zw, Pos), 0, 1);
    OutUV = lerp(UVMinMax.xy, UVMinMax.zw, Pos);

#if DEPTH_RESOLVE_TEXTUREARRAY
	OutViewId = (InstanceID & 1);
	OutRTIndex = OutViewId;
#endif
}
