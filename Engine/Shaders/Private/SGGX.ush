// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "MonteCarlo.ush"

float SGGX_ProjectedArea( float a2, float NoV )
{
	// = Integral saturate( VoM ) * SGGX_D( a2, NoM )
	return sqrt( NoV * NoV + a2 * (1 - NoV * NoV) );
}

float SGGX_SurfaceArea( float a2 )
{
	return
		(1 - rsqrt(3)) * ( sqrt(a2) * 2 + 1 ) +
		(1 + rsqrt(3)) * sqrt( a2 * 2 + 1 );
}

float SGGX_D( float a2, float NoH )
{
	float d = ( NoH * a2 - NoH ) * NoH + 1;	// 2 mad
	return a2 / ( PI*d*d );					// 4 mul, 1 rcp
}

float SGGX_Dvis( float a2, float NoV, float VoM, float NoM )
{
	return saturate( VoM ) * SGGX_D( a2, NoM ) / SGGX_ProjectedArea( a2, NoV );
}

float4 SGGX_DvisSample( float2 E, float a2, float3 TangentV )
{
	float3 a = 1;
	a.xy = sqrt(a2);

	float3 SphereV = normalize( TangentV * a );
	float3 SphereM = CosineSampleHemisphere( E, SphereV ).xyz;
	float3 TangentM = normalize( SphereM * a );

	float PDF = SGGX_Dvis( a2, TangentV.z, dot( TangentV, TangentM ), TangentM.z );
	return float4( TangentM, PDF );
}

// 3 component alpha versions of the same functions
float SGGX_ProjectedArea( float3 a, float3 V )
{
	return length( a * V );
}

float SGGX_SurfaceArea( float3 a )
{
	return
		(1 - rsqrt(3)) * dot( a, 1 ) +
		(1 + rsqrt(3)) * length(a);
}

float SGGX_D( float3 a, float3 H )
{
	float a3 = a.x * a.y * a.z;
	return (1.0f / PI) * a3 * Pow2( a3 / dot( a.yxx * H, a.zzy * H ) );
}

float SGGX_Dvis( float3 a, float3 V, float3 M )
{
	return saturate( dot( V, M ) ) * SGGX_D( a, M ) / SGGX_ProjectedArea( a, V );
}

float4 SGGX_DvisSample( float2 E, float3 a, float3 TangentV )
{
	float3 SphereV = normalize( TangentV * a );
	float3 SphereM = CosineSampleHemisphere( E, SphereV ).xyz;
	float3 TangentM = normalize( SphereM * a );

	float PDF = SGGX_Dvis( a, TangentV, TangentM );
	return float4( TangentM, PDF );
}

float SGGX_Diffuse( float VoL )
{
#if 1
	return Pow2(VoL + 1) * 0.115 + (VoL + 1) * 0.1;
#else
	float3 L = float3( sqrt( 1 - VoL * VoL ), 0, VoL );

	float Result = 0;

	const uint NumSamples = 64;
	for( uint i = 0; i < NumSamples; i++ )
	{
		RandomSequence RandSequence;
		RandomSequence_Initialize( RandSequence, View.StateFrameIndex, i );

		// visible sample on sphere
		float3 VisibleNormal = CosineSampleHemisphere( RandomSequence_GenerateSample2D( RandSequence ) ).xyz;

		Result += saturate( dot( VisibleNormal, L ) );
	}

	return Result / NumSamples;
#endif
}

float SGGX_Convolve( float Geo_a2, float Mat_a2 )
{
	float Geo_a = sqrt( Geo_a2 );
	float Mat_a = sqrt( Mat_a2 );

	//return ( Geo_a2 + Mat_a2 ) / ( 1 + Geo_a2 * Mat_a2 );
	return ( Geo_a2 + Geo_a * Mat_a + Mat_a2 ) / ( 1 + Geo_a * Mat_a + Geo_a2 * Mat_a2 );
	//return sqrt( ( Geo_a2 + Geo_a2 * Mat_a + Geo_a * Mat_a2 + Mat_a2 ) / ( 1 + Geo_a2 * Mat_a + Geo_a * Mat_a2 + Geo_a2 * Mat_a2 ) );
	//return Pow2( ( Geo_a + Mat_a ) / ( 1 + Geo_a * Mat_a ) );
}