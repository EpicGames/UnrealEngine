// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SHCommon.usf: Spherical Harmonic functionality
=============================================================================*/

#pragma once

/** The SH coefficients for the projection of a function that maps directions to scalar values. */
struct FOneBandSHVector
{
	half V;
};

/** The SH coefficients for the projection of a function that maps directions to RGB values. */
struct FOneBandSHVectorRGB
{
	FOneBandSHVector R;
	FOneBandSHVector G;
	FOneBandSHVector B;
};

/** The SH coefficients for the projection of a function that maps directions to scalar values. */
struct FTwoBandSHVector
{
	half4 V;
};

/** The SH coefficients for the projection of a function that maps directions to RGB values. */
struct FTwoBandSHVectorRGB
{
	FTwoBandSHVector R;
	FTwoBandSHVector G;
	FTwoBandSHVector B;
};

/** The SH coefficients for the projection of a function that maps directions to scalar values. */
struct FThreeBandSHVector
{
	half4 V0;
	half4 V1;
	half V2;
#if defined(NEED_SH_VECTOR_PADDING) && NEED_SH_VECTOR_PADDING==1
	// Add padding to workaround a bug with some shader compilers when storing structs in LDS
	half3 Pad;
#endif
};

struct FThreeBandSHVectorRGB
{
	FThreeBandSHVector R;
	FThreeBandSHVector G;
	FThreeBandSHVector B;
};

FTwoBandSHVectorRGB MulSH(FTwoBandSHVectorRGB A, half Scalar)
{
	FTwoBandSHVectorRGB Result;
	Result.R.V = A.R.V * Scalar;
	Result.G.V = A.G.V * Scalar;
	Result.B.V = A.B.V * Scalar;
	return Result;
}

FTwoBandSHVectorRGB MulSH(FTwoBandSHVector A, half3 Color)
{
	FTwoBandSHVectorRGB Result;
	Result.R.V = A.V * Color.r;
	Result.G.V = A.V * Color.g;
	Result.B.V = A.V * Color.b;
	return Result;
}

FTwoBandSHVector MulSH(FTwoBandSHVector A, half Scalar)
{
	FTwoBandSHVector Result;
	Result.V = A.V * Scalar;
	return Result;
}

FThreeBandSHVectorRGB MulSH3(FThreeBandSHVector A, half3 Color)
{
	FThreeBandSHVectorRGB Result;
	Result.R.V0 = A.V0 * Color.r;
	Result.R.V1 = A.V1 * Color.r;
	Result.R.V2 = A.V2 * Color.r;
	Result.G.V0 = A.V0 * Color.g;
	Result.G.V1 = A.V1 * Color.g;
	Result.G.V2 = A.V2 * Color.g;
	Result.B.V0 = A.V0 * Color.b;
	Result.B.V1 = A.V1 * Color.b;
	Result.B.V2 = A.V2 * Color.b;
	return Result;
}

FThreeBandSHVector MulSH3(FThreeBandSHVector A, half Scalar)
{
	FThreeBandSHVector Result;
	Result.V0 = A.V0 * Scalar;
	Result.V1 = A.V1 * Scalar;
	Result.V2 = A.V2 * Scalar;
	return Result;
}

FTwoBandSHVector AddSH(FTwoBandSHVector A, FTwoBandSHVector B)
{
	FTwoBandSHVector Result = A;
	Result.V += B.V;
	return Result;
}

FTwoBandSHVectorRGB AddSH(FTwoBandSHVectorRGB A, FTwoBandSHVectorRGB B)
{
	FTwoBandSHVectorRGB Result;
	Result.R = AddSH(A.R, B.R);
	Result.G = AddSH(A.G, B.G);
	Result.B = AddSH(A.B, B.B);
	return Result;
}

FThreeBandSHVector AddSH3(FThreeBandSHVector A, FThreeBandSHVector B)
{
	FThreeBandSHVector Result = A;
	Result.V0 += B.V0;
	Result.V1 += B.V1;
	Result.V2 += B.V2;
	return Result;
}

FThreeBandSHVectorRGB AddSH3(FThreeBandSHVectorRGB A, FThreeBandSHVectorRGB B)
{
	FThreeBandSHVectorRGB Result;
	Result.R = AddSH3(A.R, B.R);
	Result.G = AddSH3(A.G, B.G);
	Result.B = AddSH3(A.B, B.B);
	return Result;
}

/**
 * Computes the dot product of two SH projected scalar functions, giving a scalar value that is
 * the integral over all directions of the product of the functions.
 */
half DotSH(FTwoBandSHVector A,FTwoBandSHVector B)
{
	half Result = dot(A.V, B.V);
	return Result;
}

/**
 * Computes the dot product of a SH projected RGB function and a SH projected scalar function,
 * giving a RGB value that is the integral over all directions of the product of the functions.
 */
half3 DotSH(FTwoBandSHVectorRGB A,FTwoBandSHVector B)
{
	half3 Result = 0;
	Result.r = DotSH(A.R,B);
	Result.g = DotSH(A.G,B);
	Result.b = DotSH(A.B,B);
	return Result;
}

half DotSH1(FOneBandSHVector A,FOneBandSHVector B)
{
	half Result = A.V * B.V;
	return Result;
}

half3 DotSH1(FOneBandSHVectorRGB A,FOneBandSHVector B)
{
	half3 Result = 0;
	Result.r = DotSH1(A.R,B);
	Result.g = DotSH1(A.G,B);
	Result.b = DotSH1(A.B,B);
	return Result;
}

half DotSH3(FThreeBandSHVector A,FThreeBandSHVector B)
{
	half Result = dot(A.V0, B.V0);
	Result += dot(A.V1, B.V1);
	Result += A.V2 * B.V2;
	return Result;
}

half3 DotSH3(FThreeBandSHVectorRGB A,FThreeBandSHVector B)
{
	half3 Result = 0;
	Result.r = DotSH3(A.R,B);
	Result.g = DotSH3(A.G,B);
	Result.b = DotSH3(A.B,B);
	return Result;
}

FTwoBandSHVector GetLuminance(FTwoBandSHVectorRGB InRGBVector)
{
	half3 LF = (half3) LuminanceFactors();
	
	FTwoBandSHVector Out;
	Out.V = InRGBVector.R.V * LF.x + InRGBVector.G.V * LF.y + InRGBVector.B.V * LF.z;
	return Out;
}

/** Compute the direction which the spherical harmonic is highest at. */
float3 GetMaximumDirection(FTwoBandSHVector SHVector)
{
	// This is an approximation which only takes into account first and second order spherical harmonics.
	float3 MaxDirection = float3(-SHVector.V.w, -SHVector.V.y, SHVector.V.z);
	float Length = length(MaxDirection);
	return MaxDirection / max(Length, .0001f);
}

/** Computes the value of the SH basis functions for a given direction. */
FOneBandSHVector SHBasisFunction1()
{
	FOneBandSHVector Result;
	// These are derived from simplifying SHBasisFunction in C++
	Result.V = 0.282095f; 
	return Result;
}

FTwoBandSHVector SHBasisFunction(half3 InputVector)
{
	FTwoBandSHVector Result;
	// These are derived from simplifying SHBasisFunction in C++
	Result.V.x = 0.282095f; 
	Result.V.y = -0.488603f * InputVector.y;
	Result.V.z = 0.488603f * InputVector.z;
	Result.V.w = -0.488603f * InputVector.x;
	return Result;
}

FThreeBandSHVector SHBasisFunction3(half3 InputVector)
{
	FThreeBandSHVector Result;
	// These are derived from simplifying SHBasisFunction in C++
	Result.V0.x = 0.282095f; 
	Result.V0.y = -0.488603f * InputVector.y;
	Result.V0.z = 0.488603f * InputVector.z;
	Result.V0.w = -0.488603f * InputVector.x;

	half3 VectorSquared = InputVector * InputVector;
	Result.V1.x = 1.092548f * InputVector.x * InputVector.y;
	Result.V1.y = -1.092548f * InputVector.y * InputVector.z;
	Result.V1.z = 0.315392f * (3.0f * VectorSquared.z - 1.0f);
	Result.V1.w = -1.092548f * InputVector.x * InputVector.z;
	Result.V2 = 0.546274f * (VectorSquared.x - VectorSquared.y);

	return Result;
}

/** Simplified to only return the ambient coefficient as all the rest are 0. */
half SHAmbientFunction()
{
	return 1 / (2 * sqrt(PI));
}

/**
 * Computes the diffuse transfer function for a surface with the given normal and DiffusePower,
 * and projects it into the SH basis.
 */
FOneBandSHVector CalcDiffuseTransferSH1(half Exponent)
{
	FOneBandSHVector Result = SHBasisFunction1();

	// These formula are scaling factors for each SH band that convolve a SH with the circularly symmetric function
	// max(0,cos(theta))^Exponent
	half L0 =					2 * PI / (1 + 1 * Exponent							);

	// Multiply the coefficients in each band with the appropriate band scaling factor.
	Result.V *= L0;

	return Result;
}

FTwoBandSHVector CalcDiffuseTransferSH(half3 Normal,half Exponent)
{
	FTwoBandSHVector Result = SHBasisFunction(Normal);

	// These formula are scaling factors for each SH band that convolve a SH with the circularly symmetric function
	// max(0,cos(theta))^Exponent
	half L0 =					2 * PI / (1 + 1 * Exponent							);
	half L1 =					2 * PI / (2 + 1 * Exponent							);

	// Multiply the coefficients in each band with the appropriate band scaling factor.
	Result.V.x *= L0;
	Result.V.yzw *= L1;

	return Result;
}

FThreeBandSHVector CalcDiffuseTransferSH3(half3 Normal,half Exponent)
{
	FThreeBandSHVector Result = SHBasisFunction3(Normal);

	// These formula are scaling factors for each SH band that convolve a SH with the circularly symmetric function
	// max(0,cos(theta))^Exponent
	half L0 =					2 * PI / (1 + 1 * Exponent						);
	half L1 =					2 * PI / (2 + 1 * Exponent						);
	half L2 = Exponent *		2 * PI / (3 + 4 * Exponent + Exponent * Exponent);
	half L3 = (Exponent - 1) *	2 * PI / (8 + 6 * Exponent + Exponent * Exponent);

	// Multiply the coefficients in each band with the appropriate band scaling factor.
#if VULKAN_PROFILE
	// Workaround bug in Vulkan precision on mobile in OpVectorTimesScalar
	Result.V0.x *= float(L0);
	Result.V0.yzw *= float(L1);
	Result.V1.xyzw *= float(L2);
	Result.V2 *= float(L2);
#else
	Result.V0.x *= L0;
	Result.V0.yzw *= L1;
	Result.V1.xyzw *= L2;
	Result.V2 *= L2;
#endif
	return Result;
}

/**
 * Calculates Irradiance for a cone with the given SH lighting
 * Based on "Practical Real-Time Strategies for Accurate Indirect Occlusion"
 */
float3 EvaluateSHIrradiance(float3 Direction, float AO, FThreeBandSHVectorRGB SH)
{
	FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(Direction, /*Exponent*/ 1.0f);

	// Cos/Sin of a visibility cone aperture angle
	float CosA = sqrt(saturate(1.0f - AO));
	float SinA = sqrt(saturate(1.0f - CosA * CosA));

	// Zonal harmonics expansion
	float Z0 = SinA * SinA;
	float Z1 = (1.0f - CosA * CosA * CosA);
	float Z2 = SinA * SinA * (1.0f + 3.0f * CosA * CosA);

	DiffuseTransferSH.V0.x *= Z0;
	DiffuseTransferSH.V0.yzw *= Z1;
	DiffuseTransferSH.V1.xyzw *= Z2;
	DiffuseTransferSH.V2 *= Z2;

	return max(float3(0,0,0), DotSH3(SH, DiffuseTransferSH));
}