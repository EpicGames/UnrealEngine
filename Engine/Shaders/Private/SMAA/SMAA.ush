// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Copyright (C) 2013 Jorge Jimenez (jorge@iryoku.com)
 * Copyright (C) 2013 Jose I. Echevarria (joseignacioechevarria@gmail.com)
 * Copyright (C) 2013 Belen Masia (bmasia@unizar.es)
 * Copyright (C) 2013 Fernando Navarro (fernandn@microsoft.com)
 * Copyright (C) 2013 Diego Gutierrez (diegog@unizar.es)
 *
 * Permission is hereby granted, free of charge, to any person obtaining a copy
 * this software and associated documentation files (the "Software"), to deal in
 * the Software without restriction, including without limitation the rights to
 * use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies
 * of the Software, and to permit persons to whom the Software is furnished to
 * do so, subject to the following conditions:
 *
 * The above copyright notice and this permission notice shall be included in
 * all copies or substantial portions of the Software. As clarification, there
 * is no requirement that the copyright notice and permission be included in
 * binary distributions of the Software.
 *
 * THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
 * IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
 * FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
 * AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
 * LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
 * OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
 * SOFTWARE.
 */


/**
 *                  _______  ___  ___       ___           ___
 *                 /       ||   \/   |     /   \         /   \
 *                |   (---- |  \  /  |    /  ^  \       /  ^  \
 *                 \   \    |  |\/|  |   /  /_\  \     /  /_\  \
 *              ----)   |   |  |  |  |  /  _____  \   /  _____  \
 *             |_______/    |__|  |__| /__/     \__\ /__/     \__\
 * 
 *                               E N H A N C E D
 *       S U B P I X E L   M O R P H O L O G I C A L   A N T I A L I A S I N G
 *
 *                         http://www.iryoku.com/smaa/
 *
 * Hi, welcome aboard!
 * 
 * Here you'll find instructions to get the shader up and running as fast as
 * possible.
 *
 * IMPORTANTE NOTICE: when updating, remember to update both this file and the
 * precomputed textures! They may change from version to version.
 *
 * The shader has three passes, chained together as follows:
 *
 *                           |input|------------------·
 *                              v                     |
 *                    [ SMAA*EdgeDetection ]          |
 *                              v                     |
 *                          |edgesTex|                |
 *                              v                     |
 *              [ SMAABlendingWeightCalculation ]     |
 *                              v                     |
 *                          |blendTex|                |
 *                              v                     |
 *                [ SMAANeighborhoodBlending ] <------·
 *                              v
 *                           |output|
 *
 * Note that each [pass] has its own vertex and pixel shader. Remember to use
 * oversized triangles instead of quads to avoid overshading along the
 * diagonal.
 *
 * You've three edge detection methods to choose from: luma, color or depth.
 * They represent different quality/performance and anti-aliasing/sharpness
 * tradeoffs, so our recommendation is for you to choose the one that best
 * suits your particular scenario:
 *
 * - Depth edge detection is usually the fastest but it may miss some edges.
 *
 * - Luma edge detection is usually more expensive than depth edge detection,
 *   but catches visible edges that depth edge detection can miss.
 *
 * - Color edge detection is usually the most expensive one but catches
 *   chroma-only edges.
 *
 * For quickstarters: just use luma edge detection.
 *
 * The general advice is to not rush the integration process and ensure each
 * step is done correctly (don't try to integrate SMAA T2x with predicated edge
 * detection from the start!). Ok then, let's go!
 *
 *  1. The first step is to create two RGBA temporal render targets for holding
 *     |edgesTex| and |blendTex|.
 *
 *     In DX10 or DX11, you can use a RG render target for the edges texture.
 *     In the case of NVIDIA GPUs, using RG render targets seems to actually be
 *     slower.
 *
 *     On the Xbox 360, you can use the same render target for resolving both
 *     |edgesTex| and |blendTex|, as they aren't needed simultaneously.
 *
 *  2. Both temporal render targets |edgesTex| and |blendTex| must be cleared
 *     each frame. Do not forget to clear the alpha channel!
 *
 *  3. The next step is loading the two supporting precalculated textures,
 *     'areaTex' and 'searchTex'. You'll find them in the 'Textures' folder as
 *     C++ headers, and also as regular DDS files. They'll be needed for the
 *     'SMAABlendingWeightCalculation' pass.
 *
 *     If you use the C++ headers, be sure to load them in the format specified
 *     inside of them.
 *
 *     You can also compress 'areaTex' and 'searchTex' using BC5 and BC4
 *     respectively, if you have that option in your content processor pipeline.
 *     When compressing then, you get a non-perceptible quality decrease, and a
 *     marginal performance increase.
 *
 *  4. All samplers must be set to linear filtering and clamp.
 *
 *     After you get the technique working, remember that 64-bit inputs have
 *     half-rate linear filtering on GCN.
 *
 *     If SMAA is applied to 64-bit color buffers, switching to point filtering
 *     when accesing them will increase the performance. Search for
 *     'SMAASamplePoint' to see which textures may benefit from point
 *     filtering, and where (which is basically the color input in the edge
 *     detection and resolve passes).
 *
 *  5. All texture reads and buffer writes must be non-sRGB, with the exception
 *     of the input read and the output write in
 *     'SMAANeighborhoodBlending' (and only in this pass!). If sRGB reads in
 *     this last pass are not possible, the technique will work anyway, but
 *     will perform antialiasing in gamma space.
 *
 *     IMPORTANT: for best results the input read for the color/luma edge 
 *     detection should *NOT* be sRGB.
 *
 *  6. Before including SMAA.h you'll have to setup the render target metrics,
 *     the target and any optional configuration defines. Optionally you can
 *     use a preset.
 *
 *     You have the following targets available: 
 *         SMAA_HLSL_3
 *         SMAA_HLSL_4
 *         SMAA_HLSL_4_1
 *         SMAA_GLSL_3 *
 *         SMAA_GLSL_4 *
 *
 *         * (See SMAA_INCLUDE_VS and SMAA_INCLUDE_PS below).
 *
 *     And four presets:
 *         SMAA_PRESET_LOW          (%60 of the quality)
 *         SMAA_PRESET_MEDIUM       (%80 of the quality)
 *         SMAA_PRESET_HIGH         (%95 of the quality)
 *         SMAA_PRESET_ULTRA        (%99 of the quality)
 *
 *     For example:
 *         #define SMAA_RT_METRICS float4(1.0 / 1280.0, 1.0 / 720.0, 1280.0, 720.0)
 *         #define SMAA_HLSL_4
 *         #define SMAA_PRESET_HIGH
 *         #include "SMAA.ush"
 *
 *     Note that SMAA_RT_METRICS doesn't need to be a macro, it can be a
 *     uniform variable. The code is designed to minimize the impact of not
 *     using a constant value, but it is still better to hardcode it.
 *
 *     Depending on how you encoded 'areaTex' and 'searchTex', you may have to
 *     add (and customize) the following defines before including SMAA.h:
 *          #define SMAA_AREATEX_SELECT(sample) sample.rg
 *          #define SMAA_SEARCHTEX_SELECT(sample) sample.r
 *
 *     If your engine is already using porting macros, you can define
 *     SMAA_CUSTOM_SL, and define the porting functions by yourself.
 *
 *  7. Then, you'll have to setup the passes as indicated in the scheme above.
 *     You can take a look into SMAA.fx, to see how we did it for our demo.
 *     Checkout the function wrappers, you may want to copy-paste them!
 *
 *  8. It's recommended to validate the produced |edgesTex| and |blendTex|.
 *     You can use a screenshot from your engine to compare the |edgesTex|
 *     and |blendTex| produced inside of the engine with the results obtained
 *     with the reference demo.
 *
 *  9. After you get the last pass to work, it's time to optimize. You'll have
 *     to initialize a stencil buffer in the first pass (discard is already in
 *     the code), then mask execution by using it the second pass. The last
 *     pass should be executed in all pixels.
 *
 *
 * After this point you can choose to enable predicated thresholding,
 * temporal supersampling and motion blur integration:
 *
 * a) If you want to use predicated thresholding, take a look into
 *    SMAA_PREDICATION; you'll need to pass an extra texture in the edge
 *    detection pass.
 *
 * b) If you want to enable temporal supersampling (SMAA T2x):
 *
 * 1. The first step is to render using subpixel jitters. I won't go into
 *    detail, but it's as simple as moving each vertex position in the
 *    vertex shader, you can check how we do it in our DX10 demo.
 *
 * 2. Then, you must setup the temporal resolve. You may want to take a look
 *    into SMAAResolve for resolving 2x modes. After you get it working, you'll
 *    probably see ghosting everywhere. But fear not, you can enable the
 *    CryENGINE temporal reprojection by setting the SMAA_REPROJECTION macro.
 *    Check out SMAA_DECODE_VELOCITY if your velocity buffer is encoded.
 *
 * 3. The next step is to apply SMAA to each subpixel jittered frame, just as
 *    done for 1x.
 *
 * 4. At this point you should already have something usable, but for best
 *    results the proper area textures must be set depending on current jitter.
 *    For this, the parameter 'subsampleIndices' of
 *    'SMAABlendingWeightCalculationPS' must be set as follows, for our T2x
 *    mode:
 *
 *    @SUBSAMPLE_INDICES
 *
 *    | S# |  Camera Jitter   |  subsampleIndices    |
 *    +----+------------------+---------------------+
 *    |  0 |  ( 0.25, -0.25)  |  float4(1, 1, 1, 0)  |
 *    |  1 |  (-0.25,  0.25)  |  float4(2, 2, 2, 0)  |
 *
 *    These jitter positions assume a bottom-to-top y axis. S# stands for the
 *    sample number.
 *
 * More information about temporal supersampling here:
 *    http://iryoku.com/aacourse/downloads/13-Anti-Aliasing-Methods-in-CryENGINE-3.pdf
 *
 * c) If you want to enable spatial multisampling (SMAA S2x):
 *
 * 1. The scene must be rendered using MSAA 2x. The MSAA 2x buffer must be
 *    created with:
 *      - DX10:     see below (*)
 *      - DX10.1:   D3D10_STANDARD_MULTISAMPLE_PATTERN or
 *      - DX11:     D3D11_STANDARD_MULTISAMPLE_PATTERN
 *
 *    This allows to ensure that the subsample order matches the table in
 *    @SUBSAMPLE_INDICES.
 *
 *    (*) In the case of DX10, we refer the reader to:
 *      - SMAA::detectMSAAOrder and
 *      - SMAA::msaaReorder
 *
 *    These functions allow to match the standard multisample patterns by
 *    detecting the subsample order for a specific GPU, and reordering
 *    them appropriately.
 *
 * 2. A shader must be run to output each subsample into a separate buffer
 *    (DX10 is required). You can use SMAASeparate for this purpose, or just do
 *    it in an existing pass (for example, in the tone mapping pass, which has
 *    the advantage of feeding tone mapped subsamples to SMAA, which will yield
 *    better results).
 *
 * 3. The full SMAA 1x pipeline must be run for each separated buffer, storing
 *    the results in the final buffer. The second run should alpha blend with
 *    the existing final buffer using a blending factor of 0.5.
 *    'subsampleIndices' must be adjusted as in the SMAA T2x case (see point
 *    b).
 *
 * d) If you want to enable temporal supersampling on top of SMAA S2x
 *    (which actually is SMAA 4x):
 *
 * 1. SMAA 4x consists on temporally jittering SMAA S2x, so the first step is
 *    to calculate SMAA S2x for current frame. In this case, 'subsampleIndices'
 *    must be set as follows:
 *
 *    | F# | S# |   Camera Jitter    |    Net Jitter     |   subsampleIndices   |
 *    +----+----+--------------------+-------------------+----------------------+
 *    |  0 |  0 |  ( 0.125,  0.125)  |  ( 0.375, -0.125) |  float4(5, 3, 1, 3)  |
 *    |  0 |  1 |  ( 0.125,  0.125)  |  (-0.125,  0.375) |  float4(4, 6, 2, 3)  |
 *    +----+----+--------------------+-------------------+----------------------+
 *    |  1 |  2 |  (-0.125, -0.125)  |  ( 0.125, -0.375) |  float4(3, 5, 1, 4)  |
 *    |  1 |  3 |  (-0.125, -0.125)  |  (-0.375,  0.125) |  float4(6, 4, 2, 4)  |
 *
 *    These jitter positions assume a bottom-to-top y axis. F# stands for the
 *    frame number. S# stands for the sample number.
 *
 * 2. After calculating SMAA S2x for current frame (with the new subsample
 *    indices), previous frame must be reprojected as in SMAA T2x mode (see
 *    point b).
 *
 * e) If motion blur is used, you may want to do the edge detection pass
 *    together with motion blur. This has two advantages:
 *
 * 1. Pixels under heavy motion can be omitted from the edge detection process.
 *    For these pixels we can just store "no edge", as motion blur will take
 *    care of them.
 * 2. The center pixel tap is reused.
 *
 * Note that in this case depth testing should be used instead of stenciling,
 * as we have to write all the pixels in the motion blur pass.
 *
 * That's it!
 */

//-----------------------------------------------------------------------------
// SMAA Presets

/**
 * Note that if you use one of these presets, the following configuration
 * macros will be ignored if set in the "Configurable Defines" section.
 */

#if defined(SMAA_PRESET_LOW)
#define SMAA_THRESHOLD 0.15
#define SMAA_MAX_SEARCH_STEPS 4
#define SMAA_DISABLE_DIAG_DETECTION
#define SMAA_DISABLE_CORNER_DETECTION
#elif defined(SMAA_PRESET_MEDIUM)
#define SMAA_THRESHOLD 0.1
#define SMAA_MAX_SEARCH_STEPS 8
#define SMAA_DISABLE_DIAG_DETECTION
#define SMAA_DISABLE_CORNER_DETECTION
#elif defined(SMAA_PRESET_HIGH)
#define SMAA_THRESHOLD 0.1
#define SMAA_MAX_SEARCH_STEPS 16
#define SMAA_MAX_SEARCH_STEPS_DIAG 8
#define SMAA_CORNER_ROUNDING 25
#elif defined(SMAA_PRESET_ULTRA)
#define SMAA_THRESHOLD 0.05
#define SMAA_MAX_SEARCH_STEPS 32
#define SMAA_MAX_SEARCH_STEPS_DIAG 16
#define SMAA_CORNER_ROUNDING 25
#endif

//-----------------------------------------------------------------------------
// Configurable Defines

/**
 * SMAA_THRESHOLD specifies the threshold or sensitivity to edges.
 * Lowering this value you will be able to detect more edges at the expense of
 * performance. 
 *
 * Range: [0, 0.5]
 *   0.1 is a reasonable value, and allows to catch most visible edges.
 *   0.05 is a rather overkill value, that allows to catch 'em all.
 *
 *   If temporal supersampling is used, 0.2 could be a reasonable value, as low
 *   contrast edges are properly filtered by just 2x.
 */
#ifndef SMAA_THRESHOLD
#define SMAA_THRESHOLD 0.1
#endif

/**
 * SMAA_DEPTH_THRESHOLD specifies the threshold for depth edge detection.
 * 
 * Range: depends on the depth range of the scene.
 */
#ifndef SMAA_DEPTH_THRESHOLD
#define SMAA_DEPTH_THRESHOLD (0.1 * SMAA_THRESHOLD)
#endif

/**
 * SMAA_MAX_SEARCH_STEPS specifies the maximum steps performed in the
 * horizontal/vertical pattern searches, at each side of the pixel.
 *
 * In number of pixels, it's actually the double. So the maximum line length
 * perfectly handled by, for example 16, is 64 (by perfectly, we meant that
 * longer lines won't look as good, but still antialiased).
 *
 * Range: [0, 112]
 */
#ifndef SMAA_MAX_SEARCH_STEPS
#define SMAA_MAX_SEARCH_STEPS 16
#endif

/**
 * SMAA_MAX_SEARCH_STEPS_DIAG specifies the maximum steps performed in the
 * diagonal pattern searches, at each side of the pixel. In this case we jump
 * one pixel at time, instead of two.
 *
 * Range: [0, 20]
 *
 * On high-end machines it is cheap (between a 0.8x and 0.9x slower for 16 
 * steps), but it can have a significant impact on older machines.
 *
 * Define SMAA_DISABLE_DIAG_DETECTION to disable diagonal processing.
 */
#ifndef SMAA_MAX_SEARCH_STEPS_DIAG
#define SMAA_MAX_SEARCH_STEPS_DIAG 8
#endif

/**
 * SMAA_CORNER_ROUNDING specifies how much sharp corners will be rounded.
 *
 * Range: [0, 100]
 *
 * Define SMAA_DISABLE_CORNER_DETECTION to disable corner processing.
 */
#ifndef SMAA_CORNER_ROUNDING
#define SMAA_CORNER_ROUNDING 25
#endif

/**
 * If there is an neighbor edge that has SMAA_LOCAL_CONTRAST_FACTOR times
 * bigger contrast than current edge, current edge will be discarded.
 *
 * This allows to eliminate spurious crossing edges, and is based on the fact
 * that, if there is too much contrast in a direction, that will hide
 * perceptually contrast in the other neighbors.
 */
#ifndef SMAA_LOCAL_CONTRAST_ADAPTATION_FACTOR
#define SMAA_LOCAL_CONTRAST_ADAPTATION_FACTOR 2.0
#endif

/**
 * Predicated thresholding allows to better preserve texture details and to
 * improve performance, by decreasing the number of detected edges using an
 * additional buffer like the light accumulation buffer, object ids or even the
 * depth buffer (the depth buffer usage may be limited to indoor or short range
 * scenes).
 *
 * It locally decreases the luma or color threshold if an edge is found in an
 * additional buffer (so the global threshold can be higher).
 *
 * This method was developed by Playstation EDGE MLAA team, and used in 
 * Killzone 3, by using the light accumulation buffer. More information here:
 *     http://iryoku.com/aacourse/downloads/06-MLAA-on-PS3.pptx 
 */
#ifndef SMAA_PREDICATION
#define SMAA_PREDICATION 0
#endif

/**
 * Threshold to be used in the additional predication buffer. 
 *
 * Range: depends on the input, so you'll have to find the magic number that
 * works for you.
 */
#ifndef SMAA_PREDICATION_THRESHOLD
#define SMAA_PREDICATION_THRESHOLD 0.01
#endif

/**
 * How much to scale the global threshold used for luma or color edge
 * detection when using predication.
 *
 * Range: [1, 5]
 */
#ifndef SMAA_PREDICATION_SCALE
#define SMAA_PREDICATION_SCALE 2.0
#endif

/**
 * How much to locally decrease the threshold.
 *
 * Range: [0, 1]
 */
#ifndef SMAA_PREDICATION_STRENGTH
#define SMAA_PREDICATION_STRENGTH 0.4
#endif

/**
 * Temporal reprojection allows to remove ghosting artifacts when using
 * temporal supersampling. We use the CryEngine 3 method which also introduces
 * velocity weighting. This feature is of extreme importance for totally
 * removing ghosting. More information here:
 *    http://iryoku.com/aacourse/downloads/13-Anti-Aliasing-Methods-in-CryENGINE-3.pdf
 *
 * Note that you'll need to setup a velocity buffer for enabling reprojection.
 * For static geometry, saving the previous depth buffer is a viable
 * alternative.
 */
#ifndef SMAA_REPROJECTION
#define SMAA_REPROJECTION 0
#endif

/**
 * SMAA_REPROJECTION_WEIGHT_SCALE controls the velocity weighting. It allows to
 * remove ghosting trails behind the moving object, which are not removed by
 * just using reprojection. Using low values will exhibit ghosting, while using
 * high values will disable temporal supersampling under motion.
 *
 * Behind the scenes, velocity weighting removes temporal supersampling when
 * the velocity of the subsamples differs (meaning they are different objects).
 *
 * Range: [0, 80]
 */
#ifndef SMAA_REPROJECTION_WEIGHT_SCALE
#define SMAA_REPROJECTION_WEIGHT_SCALE 30.0
#endif

/**
 * On some compilers, discard cannot be used in vertex shaders. Thus, they need
 * to be compiled separately.
 */
#ifndef SMAA_INCLUDE_VS
#define SMAA_INCLUDE_VS 1
#endif
#ifndef SMAA_INCLUDE_PS
#define SMAA_INCLUDE_PS 1
#endif

//-----------------------------------------------------------------------------
// Texture Access Defines

#ifndef SMAA_AREATEX_SELECT
#if defined(SMAA_HLSL_3)
#define SMAA_AREATEX_SELECT(sample) sample.ra
#else
#define SMAA_AREATEX_SELECT(sample) sample.rg
#endif
#endif

#ifndef SMAA_SEARCHTEX_SELECT
#define SMAA_SEARCHTEX_SELECT(sample) sample.r
#endif

#ifndef SMAA_DECODE_VELOCITY
#define SMAA_DECODE_VELOCITY(sample) sample.rg
#endif

//-----------------------------------------------------------------------------
// Non-Configurable Defines

#define SMAA_AREATEX_MAX_DISTANCE 16
#define SMAA_AREATEX_MAX_DISTANCE_DIAG 20
#define SMAA_AREATEX_PIXEL_SIZE (1.0 / float2(160.0, 560.0))
#define SMAA_AREATEX_SUBTEX_SIZE (1.0 / 7.0)
#define SMAA_SEARCHTEX_SIZE float2(66.0, 33.0)
#define SMAA_SEARCHTEX_PACKED_SIZE float2(64.0, 16.0)
#define SMAA_CORNER_ROUNDING_NORM (float(SMAA_CORNER_ROUNDING) / 100.0)

//-----------------------------------------------------------------------------
// Porting Functions

#if defined(SMAA_HLSL_3)
#define SMAATexture2D(tex) sampler2D tex
#define SMAATexturePass2D(tex) tex
#define SMAASampleLevelZero(tex, coord) tex2Dlod(tex, float4(coord, 0.0, 0.0))
#define SMAASampleLevelZeroPoint(tex, coord) tex2Dlod(tex, float4(coord, 0.0, 0.0))
#define SMAASampleLevelZeroOffset(tex, coord, offset) tex2Dlod(tex, float4(coord + offset * SMAA_RT_METRICS.xy, 0.0, 0.0))
#define SMAASample(tex, coord) tex2D(tex, coord)
#define SMAASamplePoint(tex, coord) tex2D(tex, coord)
#define SMAASampleOffset(tex, coord, offset) tex2D(tex, coord + offset * SMAA_RT_METRICS.xy)
#define SMAA_FLATTEN [flatten]
#define SMAA_BRANCH [branch]
#endif
#if defined(SMAA_HLSL_4) || defined(SMAA_HLSL_4_1)
SamplerState LinearSampler;// { Filter = MIN_MAG_LINEAR_MIP_POINT; AddressU = Clamp; AddressV = Clamp; };
SamplerState PointSampler;// { Filter = MIN_MAG_MIP_POINT; AddressU = Clamp; AddressV = Clamp; };
#define SMAATexture2D(tex) Texture2D tex
#define SMAATexturePass2D(tex) tex
#define SMAASampleLevelZero(tex, coord) tex.SampleLevel(LinearSampler, coord, 0)
#define SMAASampleLevelZeroPoint(tex, coord) tex.SampleLevel(PointSampler, coord, 0)
#define SMAASampleLevelZeroOffset(tex, coord, offset) tex.SampleLevel(LinearSampler, coord, 0, offset)
#define SMAASample(tex, coord) tex.Sample(LinearSampler, coord)
#define SMAASamplePoint(tex, coord) tex.Sample(PointSampler, coord)
#define SMAASampleOffset(tex, coord, offset) tex.Sample(LinearSampler, coord, offset)
#define SMAA_FLATTEN [flatten]
#define SMAA_BRANCH [branch]
#define SMAATexture2DMS2(tex) Texture2DMS<float4, 2> tex
#define SMAALoad(tex, pos, sample) tex.Load(pos, sample)
#if defined(SMAA_HLSL_4_1)
#define SMAAGather(tex, coord) tex.Gather(LinearSampler, coord, 0)
#endif
#endif
#if defined(SMAA_GLSL_3) || defined(SMAA_GLSL_4)
#define SMAATexture2D(tex) sampler2D tex
#define SMAATexturePass2D(tex) tex
#define SMAASampleLevelZero(tex, coord) textureLod(tex, coord, 0.0)
#define SMAASampleLevelZeroPoint(tex, coord) textureLod(tex, coord, 0.0)
#define SMAASampleLevelZeroOffset(tex, coord, offset) textureLodOffset(tex, coord, 0.0, offset)
#define SMAASample(tex, coord) texture(tex, coord)
#define SMAASamplePoint(tex, coord) texture(tex, coord)
#define SMAASampleOffset(tex, coord, offset) texture(tex, coord, offset)
#define SMAA_FLATTEN
#define SMAA_BRANCH
#define lerp(a, b, t) mix(a, b, t)
#define saturate(a) clamp(a, 0.0, 1.0)
#if defined(SMAA_GLSL_4)
#define mad(a, b, c) fma(a, b, c)
#define SMAAGather(tex, coord) textureGather(tex, coord)
#else
#define mad(a, b, c) (a * b + c)
#endif
#define float2 vec2
#define float3 vec3
#define float4 vec4
#define int2 ivec2
#define int3 ivec3
#define int4 ivec4
#define bool2 bvec2
#define bool3 bvec3
#define bool4 bvec4
#endif

#if MOBILE_MULTI_VIEW
// MMV versions of the above defines which handle Texture2DArray color, edge, and blend textures and accept an array index
#define SMAATexture2DMMV(tex) Texture2DArray tex
#define SMAASamplePointMMV(tex, coord, index) tex.Sample(PointSampler, float3(coord, index))
#define SMAASampleLevelZeroMMV(tex, coord, index) tex.SampleLevel(LinearSampler, float3(coord, index), 0)
#define SMAASampleLevelZeroOffsetMMV(tex, coord, offset, index) tex.SampleLevel(LinearSampler, float3(coord, index), 0, offset)
#define SMAASampleMMV(tex, coord, index) tex.Sample(LinearSampler, float3(coord, index))
#else
// When MMV is disabled, these match default behavior 
#define SMAATexture2DMMV(tex) SMAATexture2D(tex)
#define SMAASamplePointMMV(tex, coord, index) SMAASamplePoint(tex, coord)
#define SMAASampleLevelZeroMMV(tex, coord, index) SMAASampleLevelZero(tex, coord)
#define SMAASampleLevelZeroOffsetMMV(tex, coord, offset, index) SMAASampleLevelZeroOffset(tex, coord, offset)
#define SMAASampleMMV(tex, coord, index) SMAASample(tex, coord)
#endif

#if !defined(SMAA_HLSL_3) && !defined(SMAA_HLSL_4) && !defined(SMAA_HLSL_4_1) && !defined(SMAA_GLSL_3) && !defined(SMAA_GLSL_4) && !defined(SMAA_CUSTOM_SL)
#error you must define the shading language: SMAA_HLSL_*, SMAA_GLSL_* or SMAA_CUSTOM_SL
#endif

//-----------------------------------------------------------------------------
// Misc functions

/**
 * Gathers current pixel, and the top-left neighbors.
 */
float3 SMAAGatherNeighbours(float2 texcoord,
                            float4 offset[3],
                            SMAATexture2D(tex)) {
    #ifdef SMAAGather
    return SMAAGather(tex, texcoord + SMAA_RT_METRICS.xy * float2(-0.5, -0.5)).grb;
    #else
    float P = SMAASamplePoint(tex, texcoord).r;
    float Pleft = SMAASamplePoint(tex, offset[0].xy).r;
    float Ptop  = SMAASamplePoint(tex, offset[0].zw).r;
    return float3(P, Pleft, Ptop);
    #endif
}

/**
 * Adjusts the threshold by means of predication.
 */
float2 SMAACalculatePredicatedThreshold(float2 texcoord,
                                        float4 offset[3],
                                        SMAATexture2D(predicationTex)) {
    float3 neighbours = SMAAGatherNeighbours(texcoord, offset, SMAATexturePass2D(predicationTex));
    float2 delta = abs(neighbours.xx - neighbours.yz);
    float2 edges = step(SMAA_PREDICATION_THRESHOLD, delta);
    return SMAA_PREDICATION_SCALE * SMAA_THRESHOLD * (1.0 - SMAA_PREDICATION_STRENGTH * edges);
}

/**
 * Conditional move:
 */
void SMAAMovc(bool2 cond, inout float2 variable, float2 value) {
    SMAA_FLATTEN if (cond.x) variable.x = value.x;
    SMAA_FLATTEN if (cond.y) variable.y = value.y;
}

void SMAAMovc(bool4 cond, inout float4 variable, float4 value) {
    SMAAMovc(cond.xy, variable.xy, value.xy);
    SMAAMovc(cond.zw, variable.zw, value.zw);
}


#if SMAA_INCLUDE_VS
//-----------------------------------------------------------------------------
// Vertex Shaders

/**
 * Edge Detection Vertex Shader
 */
void SMAAEdgeDetectionVS(float2 texcoord,
                         out float4 offset[3]) {
    offset[0] = mad(SMAA_RT_METRICS.xyxy, float4(-1.0, 0.0, 0.0, -1.0), texcoord.xyxy);
    offset[1] = mad(SMAA_RT_METRICS.xyxy, float4( 1.0, 0.0, 0.0,  1.0), texcoord.xyxy);
    offset[2] = mad(SMAA_RT_METRICS.xyxy, float4(-2.0, 0.0, 0.0, -2.0), texcoord.xyxy);
}

/**
 * Blend Weight Calculation Vertex Shader
 */
void SMAABlendingWeightCalculationVS(float2 texcoord,
                                     out float2 pixcoord,
                                     out float4 offset[3]) {
    pixcoord = texcoord * SMAA_RT_METRICS.zw;

    // We will use these offsets for the searches later on (see @PSEUDO_GATHER4):
    offset[0] = mad(SMAA_RT_METRICS.xyxy, float4(-0.25, -0.125,  1.25, -0.125), texcoord.xyxy);
    offset[1] = mad(SMAA_RT_METRICS.xyxy, float4(-0.125, -0.25, -0.125,  1.25), texcoord.xyxy);

    // And these for the searches, they indicate the ends of the loops:
    offset[2] = mad(SMAA_RT_METRICS.xxyy,
                    float4(-2.0, 2.0, -2.0, 2.0) * float(SMAA_MAX_SEARCH_STEPS),
                    float4(offset[0].xz, offset[1].yw));
}

/**
 * Neighborhood Blending Vertex Shader
 */
void SMAANeighborhoodBlendingVS(float2 texcoord,
                                out float4 offset) {
    offset = mad(SMAA_RT_METRICS.xyxy, float4( 1.0, 0.0, 0.0,  1.0), texcoord.xyxy);
}
#endif // SMAA_INCLUDE_VS

#if SMAA_INCLUDE_PS
//-----------------------------------------------------------------------------
// Edge Detection Pixel Shaders (First Pass)

/**
 * Luma Edge Detection
 *
 * IMPORTANT NOTICE: luma edge detection requires gamma-corrected colors, and
 * thus 'colorTex' should be a non-sRGB texture.
 */
float2 SMAALumaEdgeDetectionPS(float2 texcoord,
                               float4 offset[3],
                               SMAATexture2DMMV(colorTex),
                               #if SMAA_PREDICATION && !MOBILE_MULTI_VIEW
                               SMAATexture2D(predicationTex)
                               #endif
							   int ArrayIndex
                               ) {
    // Calculate the threshold:
    #if SMAA_PREDICATION && !MOBILE_MULTI_VIEW
    float2 threshold = SMAACalculatePredicatedThreshold(texcoord, offset, SMAATexturePass2D(predicationTex));
    #else
    float2 threshold = float2(SMAA_THRESHOLD, SMAA_THRESHOLD);
    #endif

    // Calculate lumas:
    float3 weights = float3(0.2126, 0.7152, 0.0722);
    float L = dot(SMAASamplePointMMV(colorTex, texcoord, ArrayIndex).rgb, weights);

    float Lleft = dot(SMAASamplePointMMV(colorTex, offset[0].xy, ArrayIndex).rgb, weights);
    float Ltop  = dot(SMAASamplePointMMV(colorTex, offset[0].zw, ArrayIndex).rgb, weights);

    // We do the usual threshold:
    float4 delta;
    delta.xy = abs(L - float2(Lleft, Ltop));
    float2 edges = step(threshold, delta.xy);

    // Then discard if there is no edge:
    if (dot(edges, float2(1.0, 1.0)) == 0.0)
        discard;

    // Calculate right and bottom deltas:
    float Lright = dot(SMAASamplePointMMV(colorTex, offset[1].xy, ArrayIndex).rgb, weights);
    float Lbottom  = dot(SMAASamplePointMMV(colorTex, offset[1].zw, ArrayIndex).rgb, weights);
    delta.zw = abs(L - float2(Lright, Lbottom));

    // Calculate the maximum delta in the direct neighborhood:
    float2 maxDelta = max(delta.xy, delta.zw);

    // Calculate left-left and top-top deltas:
    float Lleftleft = dot(SMAASamplePointMMV(colorTex, offset[2].xy, ArrayIndex).rgb, weights);
    float Ltoptop = dot(SMAASamplePointMMV(colorTex, offset[2].zw, ArrayIndex).rgb, weights);
    delta.zw = abs(float2(Lleft, Ltop) - float2(Lleftleft, Ltoptop));

    // Calculate the final maximum delta:
    maxDelta = max(maxDelta.xy, delta.zw);
    float finalDelta = max(maxDelta.x, maxDelta.y);

    // Local contrast adaptation:
    edges.xy *= step(finalDelta, SMAA_LOCAL_CONTRAST_ADAPTATION_FACTOR * delta.xy);

    return edges;
}

/**
 * Color Edge Detection
 *
 * IMPORTANT NOTICE: color edge detection requires gamma-corrected colors, and
 * thus 'colorTex' should be a non-sRGB texture.
 */
float2 SMAAColorEdgeDetectionPS(float2 texcoord,
                                float4 offset[3],
                                SMAATexture2DMMV(colorTex),
                                #if SMAA_PREDICATION && !MOBILE_MULTI_VIEW
                                SMAATexture2D(predicationTex),
                                #endif
								int ArrayIndex
                                ) {
    // Calculate the threshold:
    #if SMAA_PREDICATION && !MOBILE_MULTI_VIEW
    float2 threshold = SMAACalculatePredicatedThreshold(texcoord, offset, predicationTex);
    #else
    float2 threshold = float2(SMAA_THRESHOLD, SMAA_THRESHOLD);
    #endif

    // Calculate color deltas:
    float4 delta;
    float3 C = SMAASamplePointMMV(colorTex, texcoord, ArrayIndex).rgb;

    float3 Cleft = SMAASamplePointMMV(colorTex, offset[0].xy, ArrayIndex).rgb;
    float3 t = abs(C - Cleft);
    delta.x = max(max(t.r, t.g), t.b);

    float3 Ctop  = SMAASamplePointMMV(colorTex, offset[0].zw, ArrayIndex).rgb;
    t = abs(C - Ctop);
    delta.y = max(max(t.r, t.g), t.b);

    // We do the usual threshold:
    float2 edges = step(threshold, delta.xy);

    // Then discard if there is no edge:
    if (dot(edges, float2(1.0, 1.0)) == 0.0)
        discard;

    // Calculate right and bottom deltas:
    float3 Cright = SMAASamplePointMMV(colorTex, offset[1].xy, ArrayIndex).rgb;
    t = abs(C - Cright);
    delta.z = max(max(t.r, t.g), t.b);

    float3 Cbottom  = SMAASamplePointMMV(colorTex, offset[1].zw, ArrayIndex).rgb;
    t = abs(C - Cbottom);
    delta.w = max(max(t.r, t.g), t.b);

    // Calculate the maximum delta in the direct neighborhood:
    float2 maxDelta = max(delta.xy, delta.zw);

    // Calculate left-left and top-top deltas:
    float3 Cleftleft  = SMAASamplePointMMV(colorTex, offset[2].xy, ArrayIndex).rgb;
    t = abs(C - Cleftleft);
    delta.z = max(max(t.r, t.g), t.b);

    float3 Ctoptop = SMAASamplePointMMV(colorTex, offset[2].zw, ArrayIndex).rgb;
    t = abs(C - Ctoptop);
    delta.w = max(max(t.r, t.g), t.b);

    // Calculate the final maximum delta:
    maxDelta = max(maxDelta.xy, delta.zw);
    float finalDelta = max(maxDelta.x, maxDelta.y);

    // Local contrast adaptation:
    edges.xy *= step(finalDelta, SMAA_LOCAL_CONTRAST_ADAPTATION_FACTOR * delta.xy);

    return edges;
}

/**
 * Depth Edge Detection
 */
float2 SMAADepthEdgeDetectionPS(float2 texcoord,
                                float4 offset[3],
                                SMAATexture2D(depthTex)) {
    float3 neighbours = SMAAGatherNeighbours(texcoord, offset, SMAATexturePass2D(depthTex));
    float2 delta = abs(neighbours.xx - float2(neighbours.y, neighbours.z));
    float2 edges = step(SMAA_DEPTH_THRESHOLD, delta);

    if (dot(edges, float2(1.0, 1.0)) == 0.0)
        discard;

    return edges;
}

//-----------------------------------------------------------------------------
// Diagonal Search Functions

#if !defined(SMAA_DISABLE_DIAG_DETECTION)

/**
 * Allows to decode two binary values from a bilinear-filtered access.
 */
float2 SMAADecodeDiagBilinearAccess(float2 e) {
    // Bilinear access for fetching 'e' have a 0.25 offset, and we are
    // interested in the R and G edges:
    //
    // +---G---+-------+
    // |   x o R   x   |
    // +-------+-------+
    //
    // Then, if one of these edge is enabled:
    //   Red:   (0.75 * X + 0.25 * 1) => 0.25 or 1.0
    //   Green: (0.75 * 1 + 0.25 * X) => 0.75 or 1.0
    //
    // This function will unpack the values (mad + mul + round):
    // wolframalpha.com: round(x * abs(5 * x - 5 * 0.75)) plot 0 to 1
    e.r = e.r * abs(5.0 * e.r - 5.0 * 0.75);
    return round(e);
}

float4 SMAADecodeDiagBilinearAccess(float4 e) {
    e.rb = e.rb * abs(5.0 * e.rb - 5.0 * 0.75);
    return round(e);
}

/**
 * These functions allows to perform diagonal pattern searches.
 */
float2 SMAASearchDiag1(SMAATexture2DMMV(edgesTex), float2 texcoord, float2 dir, out float2 e, int ArrayIndex) {
    float4 coord = float4(texcoord, -1.0, 1.0);
    float3 t = float3(SMAA_RT_METRICS.xy, 1.0);
    while (coord.z < float(SMAA_MAX_SEARCH_STEPS_DIAG - 1) &&
           coord.w > 0.9) {
        coord.xyz = mad(t, float3(dir, 1.0), coord.xyz);
        e = SMAASampleLevelZeroMMV(edgesTex, coord.xy, ArrayIndex).rg;
        coord.w = dot(e, float2(0.5, 0.5));
    }
    return coord.zw;
}

float2 SMAASearchDiag2(SMAATexture2DMMV(edgesTex), float2 texcoord, float2 dir, out float2 e, int ArrayIndex) {
    float4 coord = float4(texcoord, -1.0, 1.0);
    coord.x += 0.25 * SMAA_RT_METRICS.x; // See @SearchDiag2Optimization
    float3 t = float3(SMAA_RT_METRICS.xy, 1.0);
    while (coord.z < float(SMAA_MAX_SEARCH_STEPS_DIAG - 1) &&
           coord.w > 0.9) {
        coord.xyz = mad(t, float3(dir, 1.0), coord.xyz);

        // @SearchDiag2Optimization
        // Fetch both edges at once using bilinear filtering:
        e = SMAASampleLevelZeroMMV(edgesTex, coord.xy, ArrayIndex).rg;
        e = SMAADecodeDiagBilinearAccess(e);

        // Non-optimized version:
        // e.g = SMAASampleLevelZeroMMV(edgesTex, coord.xy, ArrayIndex).g;
        // e.r = SMAASampleLevelZeroOffsetMMV(edgesTex, coord.xy, int2(1, 0), ArrayIndex).r;

        coord.w = dot(e, float2(0.5, 0.5));
    }
    return coord.zw;
}

/** 
 * Similar to SMAAArea, this calculates the area corresponding to a certain
 * diagonal distance and crossing edges 'e'.
 */
float2 SMAAAreaDiag(SMAATexture2D(areaTex), float2 dist, float2 e, float offset) {
    float2 texcoord = mad(float2(SMAA_AREATEX_MAX_DISTANCE_DIAG, SMAA_AREATEX_MAX_DISTANCE_DIAG), e, dist);

    // We do a scale and bias for mapping to texel space:
    texcoord = mad(SMAA_AREATEX_PIXEL_SIZE, texcoord, 0.5 * SMAA_AREATEX_PIXEL_SIZE);

    // Diagonal areas are on the second half of the texture:
    texcoord.x += 0.5;

    // Move to proper place, according to the subpixel offset:
    texcoord.y += SMAA_AREATEX_SUBTEX_SIZE * offset;

    // Do it!
    return SMAA_AREATEX_SELECT(SMAASampleLevelZero(areaTex, texcoord));
}

/**
 * This searches for diagonal patterns and returns the corresponding weights.
 */
float2 SMAACalculateDiagWeights(SMAATexture2DMMV(edgesTex), SMAATexture2D(areaTex), float2 texcoord, float2 e, float4 subsampleIndices, int ArrayIndex) {
    float2 weights = float2(0.0, 0.0);

    // Search for the line ends:
    float4 d;
    float2 end;
    if (e.r > 0.0) {
        d.xz = SMAASearchDiag1(SMAATexturePass2D(edgesTex), texcoord, float2(-1.0,  1.0), end, ArrayIndex);
        d.x += float(end.y > 0.9);
    } else
        d.xz = float2(0.0, 0.0);
    d.yw = SMAASearchDiag1(SMAATexturePass2D(edgesTex), texcoord, float2(1.0, -1.0), end, ArrayIndex);

    SMAA_BRANCH
    if (d.x + d.y > 2.0) { // d.x + d.y + 1 > 3
        // Fetch the crossing edges:
        float4 coords = mad(float4(-d.x + 0.25, d.x, d.y, -d.y - 0.25), SMAA_RT_METRICS.xyxy, texcoord.xyxy);
        float4 c;
        c.xy = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xy, int2(-1,  0), ArrayIndex).rg;
        c.zw = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.zw, int2( 1,  0), ArrayIndex).rg;
        c.yxwz = SMAADecodeDiagBilinearAccess(c.xyzw);

        // Non-optimized version:
        // float4 coords = mad(float4(-d.x, d.x, d.y, -d.y), SMAA_RT_METRICS.xyxy, texcoord.xyxy);
        // float4 c;
        // c.x = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xy, int2(-1,  0), ArrayIndex).g;
        // c.y = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xy, int2( 0,  0), ArrayIndex).r;
        // c.z = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.zw, int2( 1,  0), ArrayIndex).g;
        // c.w = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.zw, int2( 1, -1), ArrayIndex).r;

        // Merge crossing edges at each side into a single value:
        float2 cc = mad(float2(2.0, 2.0), c.xz, c.yw);

        // Remove the crossing edge if we didn't found the end of the line:
        SMAAMovc(bool2(step(0.9, d.zw)), cc, float2(0.0, 0.0));

        // Fetch the areas for this line:
        weights += SMAAAreaDiag(SMAATexturePass2D(areaTex), d.xy, cc, subsampleIndices.z);
    }

    // Search for the line ends:
    d.xz = SMAASearchDiag2(SMAATexturePass2D(edgesTex), texcoord, float2(-1.0, -1.0), end, ArrayIndex);
    if (SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord, int2(1, 0), ArrayIndex).r > 0.0) {
        d.yw = SMAASearchDiag2(SMAATexturePass2D(edgesTex), texcoord, float2(1.0, 1.0), end, ArrayIndex);
        d.y += float(end.y > 0.9);
    } else
        d.yw = float2(0.0, 0.0);

    SMAA_BRANCH
    if (d.x + d.y > 2.0) { // d.x + d.y + 1 > 3
        // Fetch the crossing edges:
        float4 coords = mad(float4(-d.x, -d.x, d.y, d.y), SMAA_RT_METRICS.xyxy, texcoord.xyxy);
        float4 c;
        c.x  = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xy, int2(-1,  0), ArrayIndex).g;
        c.y  = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xy, int2( 0, -1), ArrayIndex).r;
        c.zw = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.zw, int2( 1,  0), ArrayIndex).gr;
        float2 cc = mad(float2(2.0, 2.0), c.xz, c.yw);

        // Remove the crossing edge if we didn't found the end of the line:
        SMAAMovc(bool2(step(0.9, d.zw)), cc, float2(0.0, 0.0));

        // Fetch the areas for this line:
        weights += SMAAAreaDiag(SMAATexturePass2D(areaTex), d.xy, cc, subsampleIndices.w).gr;
    }

    return weights;
}
#endif

//-----------------------------------------------------------------------------
// Horizontal/Vertical Search Functions

/**
 * This allows to determine how much length should we add in the last step
 * of the searches. It takes the bilinearly interpolated edge (see 
 * @PSEUDO_GATHER4), and adds 0, 1 or 2, depending on which edges and
 * crossing edges are active.
 */
float SMAASearchLength(SMAATexture2D(searchTex), float2 e, float offset) {
    // The texture is flipped vertically, with left and right cases taking half
    // of the space horizontally:
    float2 scale = SMAA_SEARCHTEX_SIZE * float2(0.5, -1.0);
    float2 bias = SMAA_SEARCHTEX_SIZE * float2(offset, 1.0);

    // Scale and bias to access texel centers:
    scale += float2(-1.0,  1.0);
    bias  += float2( 0.5, -0.5);

    // Convert from pixel coordinates to texcoords:
    // (We use SMAA_SEARCHTEX_PACKED_SIZE because the texture is cropped)
    scale *= 1.0 / SMAA_SEARCHTEX_PACKED_SIZE;
    bias *= 1.0 / SMAA_SEARCHTEX_PACKED_SIZE;

    // Lookup the search texture:
    return SMAA_SEARCHTEX_SELECT(SMAASampleLevelZero(searchTex, mad(scale, e, bias)));
}

/**
 * Horizontal/vertical search functions for the 2nd pass.
 */
float SMAASearchXLeft(SMAATexture2DMMV(edgesTex), SMAATexture2D(searchTex), float2 texcoord, float end, int ArrayIndex) {
    /**
     * @PSEUDO_GATHER4
     * This texcoord has been offset by (-0.25, -0.125) in the vertex shader to
     * sample between edge, thus fetching four edges in a row.
     * Sampling with different offsets in each direction allows to disambiguate
     * which edges are active from the four fetched ones.
     */
    float2 e = float2(0.0, 1.0);
    while (texcoord.x > end && 
           e.g > 0.8281 && // Is there some edge not activated?
           e.r == 0.0) { // Or is there a crossing edge that breaks the line?
        e = SMAASampleLevelZeroMMV(edgesTex, texcoord, ArrayIndex).rg;
        texcoord = mad(-float2(2.0, 0.0), SMAA_RT_METRICS.xy, texcoord);
    }

    float offset = mad(-(255.0 / 127.0), SMAASearchLength(SMAATexturePass2D(searchTex), e, 0.0), 3.25);
    return mad(SMAA_RT_METRICS.x, offset, texcoord.x);

    // Non-optimized version:
    // We correct the previous (-0.25, -0.125) offset we applied:
    // texcoord.x += 0.25 * SMAA_RT_METRICS.x;

    // The searches are bias by 1, so adjust the coords accordingly:
    // texcoord.x += SMAA_RT_METRICS.x;

    // Disambiguate the length added by the last step:
    // texcoord.x += 2.0 * SMAA_RT_METRICS.x; // Undo last step
    // texcoord.x -= SMAA_RT_METRICS.x * (255.0 / 127.0) * SMAASearchLength(SMAATexturePass2D(searchTex), e, 0.0);
    // return mad(SMAA_RT_METRICS.x, offset, texcoord.x);
}

float SMAASearchXRight(SMAATexture2DMMV(edgesTex), SMAATexture2D(searchTex), float2 texcoord, float end, int ArrayIndex) {
    float2 e = float2(0.0, 1.0);
    while (texcoord.x < end && 
           e.g > 0.8281 && // Is there some edge not activated?
           e.r == 0.0) { // Or is there a crossing edge that breaks the line?
        e = SMAASampleLevelZeroMMV(edgesTex, texcoord, ArrayIndex).rg;
        texcoord = mad(float2(2.0, 0.0), SMAA_RT_METRICS.xy, texcoord);
    }
    float offset = mad(-(255.0 / 127.0), SMAASearchLength(SMAATexturePass2D(searchTex), e, 0.5), 3.25);
    return mad(-SMAA_RT_METRICS.x, offset, texcoord.x);
}

float SMAASearchYUp(SMAATexture2DMMV(edgesTex), SMAATexture2D(searchTex), float2 texcoord, float end, int ArrayIndex) {
    float2 e = float2(1.0, 0.0);
    while (texcoord.y > end && 
           e.r > 0.8281 && // Is there some edge not activated?
           e.g == 0.0) { // Or is there a crossing edge that breaks the line?
        e = SMAASampleLevelZeroMMV(edgesTex, texcoord, ArrayIndex).rg;
        texcoord = mad(-float2(0.0, 2.0), SMAA_RT_METRICS.xy, texcoord);
    }
    float offset = mad(-(255.0 / 127.0), SMAASearchLength(SMAATexturePass2D(searchTex), e.gr, 0.0), 3.25);
    return mad(SMAA_RT_METRICS.y, offset, texcoord.y);
}

float SMAASearchYDown(SMAATexture2DMMV(edgesTex), SMAATexture2D(searchTex), float2 texcoord, float end, int ArrayIndex) {
    float2 e = float2(1.0, 0.0);
    while (texcoord.y < end && 
           e.r > 0.8281 && // Is there some edge not activated?
           e.g == 0.0) { // Or is there a crossing edge that breaks the line?
        e = SMAASampleLevelZeroMMV(edgesTex, texcoord, ArrayIndex).rg;
        texcoord = mad(float2(0.0, 2.0), SMAA_RT_METRICS.xy, texcoord);
    }
    float offset = mad(-(255.0 / 127.0), SMAASearchLength(SMAATexturePass2D(searchTex), e.gr, 0.5), 3.25);
    return mad(-SMAA_RT_METRICS.y, offset, texcoord.y);
}

/** 
 * Ok, we have the distance and both crossing edges. So, what are the areas
 * at each side of current edge?
 */
float2 SMAAArea(SMAATexture2D(areaTex), float2 dist, float e1, float e2, float offset) {
    // Rounding prevents precision errors of bilinear filtering:
    float2 texcoord = mad(float2(SMAA_AREATEX_MAX_DISTANCE, SMAA_AREATEX_MAX_DISTANCE), round(4.0 * float2(e1, e2)), dist);
    
    // We do a scale and bias for mapping to texel space:
    texcoord = mad(SMAA_AREATEX_PIXEL_SIZE, texcoord, 0.5 * SMAA_AREATEX_PIXEL_SIZE);

    // Move to proper place, according to the subpixel offset:
    texcoord.y = mad(SMAA_AREATEX_SUBTEX_SIZE, offset, texcoord.y);

    // Do it!
    return SMAA_AREATEX_SELECT(SMAASampleLevelZero(areaTex, texcoord));
}

//-----------------------------------------------------------------------------
// Corner Detection Functions

void SMAADetectHorizontalCornerPattern(SMAATexture2DMMV(edgesTex), inout float2 weights, float4 texcoord, float2 d, int ArrayIndex) {
    #if !defined(SMAA_DISABLE_CORNER_DETECTION)
    float2 leftRight = step(d.xy, d.yx);
    float2 rounding = (1.0 - SMAA_CORNER_ROUNDING_NORM) * leftRight;

    rounding /= leftRight.x + leftRight.y; // Reduce blending for pixels in the center of a line.

    float2 factor = float2(1.0, 1.0);
    factor.x -= rounding.x * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.xy, int2(0,  1), ArrayIndex).r;
    factor.x -= rounding.y * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.zw, int2(1,  1), ArrayIndex).r;
    factor.y -= rounding.x * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.xy, int2(0, -2), ArrayIndex).r;
    factor.y -= rounding.y * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.zw, int2(1, -2), ArrayIndex).r;

    weights *= saturate(factor);
    #endif
}

void SMAADetectVerticalCornerPattern(SMAATexture2DMMV(edgesTex), inout float2 weights, float4 texcoord, float2 d, int ArrayIndex) {
    #if !defined(SMAA_DISABLE_CORNER_DETECTION)
    float2 leftRight = step(d.xy, d.yx);
    float2 rounding = (1.0 - SMAA_CORNER_ROUNDING_NORM) * leftRight;

    rounding /= leftRight.x + leftRight.y;

    float2 factor = float2(1.0, 1.0);
    factor.x -= rounding.x * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.xy, int2( 1, 0), ArrayIndex).g;
    factor.x -= rounding.y * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.zw, int2( 1, 1), ArrayIndex).g;
    factor.y -= rounding.x * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.xy, int2(-2, 0), ArrayIndex).g;
    factor.y -= rounding.y * SMAASampleLevelZeroOffsetMMV(edgesTex, texcoord.zw, int2(-2, 1), ArrayIndex).g;

    weights *= saturate(factor);
    #endif
}

//-----------------------------------------------------------------------------
// Blending Weight Calculation Pixel Shader (Second Pass)

float4 SMAABlendingWeightCalculationPS(float2 texcoord,
                                       float2 pixcoord,
                                       float4 offset[3],
                                       SMAATexture2DMMV(edgesTex),
                                       SMAATexture2D(areaTex),
                                       SMAATexture2D(searchTex),
                                       float4 subsampleIndices, // Just pass zero for SMAA 1x, see @SUBSAMPLE_INDICES.
									   int ArrayIndex) { 
    float4 weights = float4(0.0, 0.0, 0.0, 0.0);

    float2 e = SMAASampleMMV(edgesTex, texcoord, ArrayIndex).rg;

    SMAA_BRANCH
    if (e.g > 0.0) { // Edge at north
        #if !defined(SMAA_DISABLE_DIAG_DETECTION)
        // Diagonals have both north and west edges, so searching for them in
        // one of the boundaries is enough.
        weights.rg = SMAACalculateDiagWeights(SMAATexturePass2D(edgesTex), SMAATexturePass2D(areaTex), texcoord, e, subsampleIndices, ArrayIndex);

        // We give priority to diagonals, so if we find a diagonal we skip 
        // horizontal/vertical processing.
        SMAA_BRANCH
        if (weights.r == -weights.g) { // weights.r + weights.g == 0.0
        #endif

        float2 d;

        // Find the distance to the left:
        float3 coords;
        coords.x = SMAASearchXLeft(SMAATexturePass2D(edgesTex), SMAATexturePass2D(searchTex), offset[0].xy, offset[2].x, ArrayIndex);
        coords.y = offset[1].y; // offset[1].y = texcoord.y - 0.25 * SMAA_RT_METRICS.y (@CROSSING_OFFSET)
        d.x = coords.x;

        // Now fetch the left crossing edges, two at a time using bilinear
        // filtering. Sampling at -0.25 (see @CROSSING_OFFSET) enables to
        // discern what value each edge has:
        float e1 = SMAASampleLevelZeroMMV(edgesTex, coords.xy, ArrayIndex).r;

        // Find the distance to the right:
        coords.z = SMAASearchXRight(SMAATexturePass2D(edgesTex), SMAATexturePass2D(searchTex), offset[0].zw, offset[2].y, ArrayIndex);
        d.y = coords.z;

        // We want the distances to be in pixel units (doing this here allow to
        // better interleave arithmetic and memory accesses):
        d = abs(round(mad(SMAA_RT_METRICS.zz, d, -pixcoord.xx)));

        // SMAAArea below needs a sqrt, as the areas texture is compressed
        // quadratically:
        float2 sqrt_d = sqrt(d);

        // Fetch the right crossing edges:
        float e2 = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.zy, int2(1, 0), ArrayIndex).r;

        // Ok, we know how this pattern looks like, now it is time for getting
        // the actual area:
        weights.rg = SMAAArea(SMAATexturePass2D(areaTex), sqrt_d, e1, e2, subsampleIndices.y);

        // Fix corners:
        coords.y = texcoord.y;
        SMAADetectHorizontalCornerPattern(SMAATexturePass2D(edgesTex), weights.rg, coords.xyzy, d, ArrayIndex);

        #if !defined(SMAA_DISABLE_DIAG_DETECTION)
        } else
            e.r = 0.0; // Skip vertical processing.
        #endif
    }

    SMAA_BRANCH
    if (e.r > 0.0) { // Edge at west
        float2 d;

        // Find the distance to the top:
        float3 coords;
        coords.y = SMAASearchYUp(SMAATexturePass2D(edgesTex), SMAATexturePass2D(searchTex), offset[1].xy, offset[2].z, ArrayIndex);
        coords.x = offset[0].x; // offset[1].x = texcoord.x - 0.25 * SMAA_RT_METRICS.x;
        d.x = coords.y;

        // Fetch the top crossing edges:
        float e1 = SMAASampleLevelZeroMMV(edgesTex, coords.xy, ArrayIndex).g;

        // Find the distance to the bottom:
        coords.z = SMAASearchYDown(SMAATexturePass2D(edgesTex), SMAATexturePass2D(searchTex), offset[1].zw, offset[2].w, ArrayIndex);
        d.y = coords.z;

        // We want the distances to be in pixel units:
        d = abs(round(mad(SMAA_RT_METRICS.ww, d, -pixcoord.yy)));

        // SMAAArea below needs a sqrt, as the areas texture is compressed 
        // quadratically:
        float2 sqrt_d = sqrt(d);

        // Fetch the bottom crossing edges:
        float e2 = SMAASampleLevelZeroOffsetMMV(edgesTex, coords.xz, int2(0, 1), ArrayIndex).g;

        // Get the area for this direction:
        weights.ba = SMAAArea(SMAATexturePass2D(areaTex), sqrt_d, e1, e2, subsampleIndices.x);

        // Fix corners:
        coords.x = texcoord.x;
        SMAADetectVerticalCornerPattern(SMAATexturePass2D(edgesTex), weights.ba, coords.xyxz, d, ArrayIndex);
    }

    return weights;
}

//-----------------------------------------------------------------------------
// Neighborhood Blending Pixel Shader (Third Pass)

float4 SMAANeighborhoodBlendingPS(float2 texcoord,
                                  float4 offset,
                                  SMAATexture2DMMV(colorTex),
                                  SMAATexture2DMMV(blendTex)
                                  #if SMAA_REPROJECTION
                                  , SMAATexture2D(velocityTex)
                                  #endif
								  , int ArrayIndex
                                  ) {
    // Fetch the blending weights for current pixel:
    float4 a;
    a.x = SMAASampleMMV(blendTex, offset.xy, ArrayIndex).a; // Right
    a.y = SMAASampleMMV(blendTex, offset.zw, ArrayIndex).g; // Top
    a.wz = SMAASampleMMV(blendTex, texcoord, ArrayIndex).xz; // Bottom / Left

    // Is there any blending weight with a value greater than 0.0?
    SMAA_BRANCH
    if (dot(a, float4(1.0, 1.0, 1.0, 1.0)) < 1e-5) {
        float4 color = SMAASampleLevelZeroMMV(colorTex, texcoord, ArrayIndex);

        #if SMAA_REPROJECTION && !MOBILE_MULTI_VIEW
        float2 velocity = SMAA_DECODE_VELOCITY(SMAASampleLevelZero(velocityTex, texcoord));

        // Pack velocity into the alpha channel:
        color.a = sqrt(5.0 * length(velocity));
        #endif

        return color;
    } else {
        bool h = max(a.x, a.z) > max(a.y, a.w); // max(horizontal) > max(vertical)

        // Calculate the blending offsets:
        float4 blendingOffset = float4(0.0, a.y, 0.0, a.w);
        float2 blendingWeight = a.yw;
        SMAAMovc(bool4(h, h, h, h), blendingOffset, float4(a.x, 0.0, a.z, 0.0));
        SMAAMovc(bool2(h, h), blendingWeight, a.xz);
        blendingWeight /= dot(blendingWeight, float2(1.0, 1.0));

        // Calculate the texture coordinates:
        float4 blendingCoord = mad(blendingOffset, float4(SMAA_RT_METRICS.xy, -SMAA_RT_METRICS.xy), texcoord.xyxy);

        // We exploit bilinear filtering to mix current pixel with the chosen
        // neighbor:
        float4 color = blendingWeight.x * SMAASampleLevelZeroMMV(colorTex, blendingCoord.xy, ArrayIndex);
        color += blendingWeight.y * SMAASampleLevelZeroMMV(colorTex, blendingCoord.zw, ArrayIndex);

        #if SMAA_REPROJECTION && !MOBILE_MULTI_VIEW
        // Antialias velocity for proper reprojection in a later stage:
        float2 velocity = blendingWeight.x * SMAA_DECODE_VELOCITY(SMAASampleLevelZero(velocityTex, blendingCoord.xy));
        velocity += blendingWeight.y * SMAA_DECODE_VELOCITY(SMAASampleLevelZero(velocityTex, blendingCoord.zw));

        // Pack velocity into the alpha channel:
        color.a = sqrt(5.0 * length(velocity));
        #endif

        return color;
    }
}

//-----------------------------------------------------------------------------
// Temporal Resolve Pixel Shader (Optional Pass)

float4 SMAAResolvePS(float2 texcoord,
                     SMAATexture2D(currentColorTex),
                     SMAATexture2D(previousColorTex)
                     #if SMAA_REPROJECTION
                     , SMAATexture2D(velocityTex)
                     #endif
                     ) {
    #if SMAA_REPROJECTION
    // Velocity is assumed to be calculated for motion blur, so we need to
    // inverse it for reprojection:
    float2 velocity = -SMAA_DECODE_VELOCITY(SMAASamplePoint(velocityTex, texcoord).rg);

    // Fetch current pixel:
    float4 current = SMAASamplePoint(currentColorTex, texcoord);

    // Reproject current coordinates and fetch previous pixel:
    float4 previous = SMAASamplePoint(previousColorTex, texcoord + velocity);

    // Attenuate the previous pixel if the velocity is different:
    float delta = abs(current.a * current.a - previous.a * previous.a) / 5.0;
    float weight = 0.5 * saturate(1.0 - sqrt(delta) * SMAA_REPROJECTION_WEIGHT_SCALE);

    // Blend the pixels according to the calculated weight:
    return lerp(current, previous, weight);
    #else
    // Just blend the pixels:
    float4 current = SMAASamplePoint(currentColorTex, texcoord);
    float4 previous = SMAASamplePoint(previousColorTex, texcoord);
    return lerp(current, previous, 0.5);
    #endif
}

//-----------------------------------------------------------------------------
// Separate Multisamples Pixel Shader (Optional Pass)

#ifdef SMAALoad
void SMAASeparatePS(float4 position,
                    float2 texcoord,
                    out float4 target0,
                    out float4 target1,
                    SMAATexture2DMS2(colorTexMS)) {
    int2 pos = int2(position.xy);
    target0 = SMAALoad(colorTexMS, pos, 0);
    target1 = SMAALoad(colorTexMS, pos, 1);
}
#endif

//-----------------------------------------------------------------------------
#endif // SMAA_INCLUDE_PS
