// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMAACommon.ush"

struct FSMAABlendingWeightCalculationVertexOutput
{
	noperspective float2 UV : TEXCOORD0;
	float4 Offset[3] : TEXCOORD1;
	float2 Pixcoord : TEXCOORD4;
	float4 Position : SV_POSITION;
};

Texture2D AreaTex;
Texture2D SearchTex;

/*-----------------------------------------------------------------------------
	vertex shader
-----------------------------------------------------------------------------*/
#ifdef MainVS

	void MainVS(
		float2 InPosition : ATTRIBUTE0,
		float2 InUV       : ATTRIBUTE1,
		in FStereoVSInput StereoInput,
		out FStereoVSOutput StereoOutput,
		out FSMAABlendingWeightCalculationVertexOutput Output
	)
	{
		StereoSetupVS(StereoInput, StereoOutput);

		DrawRectangle( float4( InPosition, 0, 1 ), InUV, Output.Position, Output.UV);

		SMAABlendingWeightCalculationVS(Output.UV, Output.Pixcoord, Output.Offset);
	}

#endif // MainVS


/*-----------------------------------------------------------------------------
	pixel shader
-----------------------------------------------------------------------------*/
#ifdef MainPS

	void MainPS(
		FSMAABlendingWeightCalculationVertexOutput Input,
		in FStereoPSInput StereoInput,
		out float4 OutColor : SV_Target0)
	{
		int ArrayIndex = GetEyeIndex(StereoInput); // If using mobile multi-view (MMV), input edge texture is a texture array

		OutColor = SMAABlendingWeightCalculationPS(Input.UV, Input.Pixcoord, Input.Offset, Input_Texture, AreaTex, SearchTex, 0, ArrayIndex);
	}

#endif // MainPS
