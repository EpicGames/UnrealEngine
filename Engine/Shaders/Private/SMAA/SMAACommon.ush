// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ScreenPass.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)

#if MOBILE_MULTI_VIEW
Texture2DArray Input_Texture;
#else
Texture2D Input_Texture;
#endif
SamplerState Input_Sampler;

float4 RTMetrics;

#if SMAA_QUALITY == 0
#define SMAA_PRESET_LOW
#elif SMAA_QUALITY == 1
#define SMAA_PRESET_MEDIUM
#elif SMAA_QUALITY == 2
#define SMAA_PRESET_HIGH
#elif SMAA_QUALITY == 3
#define SMAA_PRESET_ULTRA
#endif

#define SMAA_HLSL_4
#define SMAA_RT_METRICS RTMetrics
#include "SMAA.ush"

