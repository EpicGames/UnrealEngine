// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMAACommon.ush"

struct FSMAADebugVisualizationVertexOutput
{
	noperspective float2 UV : TEXCOORD0;
	float4 Offset : TEXCOORD1;
	float4 Position : SV_POSITION;
};

#if MOBILE_MULTI_VIEW
Texture2DArray EdgeTex;
Texture2DArray BlendTex;
#else
Texture2D EdgeTex;
Texture2D BlendTex;
#endif

int DebugMode;

/*-----------------------------------------------------------------------------
	vertex shader
-----------------------------------------------------------------------------*/
#ifdef MainVS

	void MainVS(
		float2 InPosition : ATTRIBUTE0,
		float2 InUV       : ATTRIBUTE1,
		in FStereoVSInput StereoInput,
		out FStereoVSOutput StereoOutput,
		out FSMAADebugVisualizationVertexOutput Output
	)
	{
		StereoSetupVS(StereoInput, StereoOutput);

		DrawRectangle( float4( InPosition, 0, 1 ), InUV, Output.Position, Output.UV);

		SMAANeighborhoodBlendingVS(Output.UV, Output.Offset);
	}

#endif // MainVS


/*-----------------------------------------------------------------------------
	pixel shader
-----------------------------------------------------------------------------*/
#ifdef MainPS

	void MainPS(
		FSMAADebugVisualizationVertexOutput Input,
		in FStereoPSInput StereoInput,
		out float4 OutColor : SV_Target0)
	{
		int ArrayIndex = GetEyeIndex(StereoInput);

		float4 color = float4(0,0,0,0);

	#if MOBILE_MULTI_VIEW
		float3 IUV = float3(Input.UV, ArrayIndex);
	#else
		float2 IUV = Input.UV;
	#endif

		if (DebugMode == 1)
		{
			color = EdgeTex.Sample(LinearSampler, IUV);
		}
		else if (DebugMode == 2)
		{
			color = BlendTex.Sample(LinearSampler, IUV);
		}
		else if (DebugMode == 3)
		{
			// sampling logic taken from SMAANeighborhoodBlendingPS, in SMAA.ush
	
			// Fetch the blending weights for current pixel:
			float4 a;
		#if MOBILE_MULTI_VIEW
			a.x = BlendTex.Sample(LinearSampler, float3(Input.Offset.xy, ArrayIndex)).a; // Right
			a.y = BlendTex.Sample(LinearSampler, float3(Input.Offset.zw, ArrayIndex)).g; // Top
		#else
			a.x = BlendTex.Sample(LinearSampler, Input.Offset.xy).a; // Right
			a.y = BlendTex.Sample(LinearSampler, Input.Offset.zw).g; // Top
		#endif
			a.wz = BlendTex.Sample(LinearSampler, IUV).xz; // Bottom / Left

			// Is there any blending weight with a value greater than 0.0?
			if (dot(a, float4(1.0, 1.0, 1.0, 1.0)) < 1e-5)
			{
				color = Input_Texture.SampleLevel(LinearSampler, IUV, 0);
			}
			else
			{
				color = float4(1.0, 0.0, 0.0, 0.0);
			}
		}

		OutColor = color;
	}

#endif // MainPS
