// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMAACommon.ush"

struct FSMAAEdgeDetectionScreenVertexOutput
{
	noperspective float2 UV : TEXCOORD0;
	float4 Offset[3] : TEXCOORD1;
	float4 Position : SV_POSITION;
};

int EdgeMode;

/*-----------------------------------------------------------------------------
	vertex shader
-----------------------------------------------------------------------------*/
#ifdef MainVS

	void MainVS(
		float2 InPosition : ATTRIBUTE0,
		float2 InUV       : ATTRIBUTE1,
		in FStereoVSInput StereoInput,
		out FStereoVSOutput StereoOutput,
		out FSMAAEdgeDetectionScreenVertexOutput Output
	)
	{
		StereoSetupVS(StereoInput, StereoOutput);

		DrawRectangle( float4( InPosition, 0, 1 ), InUV, Output.Position, Output.UV);

		SMAAEdgeDetectionVS(Output.UV, Output.Offset);
	}

#endif // MainVS


/*-----------------------------------------------------------------------------
	pixel shader
-----------------------------------------------------------------------------*/
#ifdef MainPS

	void MainPS(
		FSMAAEdgeDetectionScreenVertexOutput Input,
		in FStereoPSInput StereoInput,
		out float2 OutColor : SV_Target0)
	{
		int ArrayIndex = GetEyeIndex(StereoInput); // If using mobile multi-view (MMV), input color is a texture array

		BRANCH
		if (EdgeMode == 0) // Color
		{
			OutColor.rg = SMAAColorEdgeDetectionPS(Input.UV, Input.Offset, Input_Texture, ArrayIndex);
		}
		else // if (EdgeMode == 1) // Luma (uses Linear Color as input texture)
		{
			OutColor.rg = SMAALumaEdgeDetectionPS(Input.UV, Input.Offset, Input_Texture, ArrayIndex);
		}
	}

#endif // MainPS
