// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMAACommon.ush"

struct FSMAANeighborhoodBlendingScreenVertexOutput
{
	noperspective float2 UV : TEXCOORD0;
	float4 Offset : TEXCOORD1;
	float4 Position : SV_POSITION;
};

#if MOBILE_MULTI_VIEW
Texture2DArray BlendTex;
#else
Texture2D BlendTex;
#endif

/*-----------------------------------------------------------------------------
	vertex shader
-----------------------------------------------------------------------------*/
#ifdef MainVS

	void MainVS(
		float2 InPosition : ATTRIBUTE0,
		float2 InUV       : ATTRIBUTE1,
		in FStereoVSInput StereoInput,
		out FStereoVSOutput StereoOutput,
		out FSMAANeighborhoodBlendingScreenVertexOutput Output
	)
	{
		StereoSetupVS(StereoInput, StereoOutput);

		DrawRectangle( float4( InPosition, 0, 1 ), InUV, Output.Position, Output.UV);

		SMAANeighborhoodBlendingVS(Output.UV, Output.Offset);
	}

#endif // MainVS


/*-----------------------------------------------------------------------------
	pixel shader
-----------------------------------------------------------------------------*/
#ifdef MainPS

	void MainPS(
		FSMAANeighborhoodBlendingScreenVertexOutput Input,
		in FStereoPSInput StereoInput,
		out float4 OutColor : SV_Target0)
	{
		int ArrayIndex = GetEyeIndex(StereoInput); // If using mobile multi-view (MMV), input blend texture is a texture array

		OutColor = SMAANeighborhoodBlendingPS(Input.UV, Input.Offset, Input_Texture, BlendTex, ArrayIndex);

	#if DIM_ALPHA_CHANNEL == 0
		OutColor.a = 1.0;
	#endif

	}

#endif // MainPS
