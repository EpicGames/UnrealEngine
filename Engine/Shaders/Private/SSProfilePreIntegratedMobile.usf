// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SSProfilePreIntegratedMobile.usf
=============================================================================*/

#include "Common.ush"
#include "BurleyNormalizedSSSCommon.ush"

uint		 SourceSubsurfaceProfileInt;
float4		 TargetSSProfilesPreIntegratedTextureSizeAndInvSize;
float4		 SourceSSProfilesTextureSizeAndInvSize;
Texture2D	 SourceSSProfilesTexture;
SamplerState SourceSSProfilesSampler;

float4 GetSourceSubsurfaceProfileTexture(uint InSampleIndex, uint InSubsurfaceProfileInt)
{
	return GetSubsurfaceProfileTexture(SourceSSProfilesTexture, SourceSSProfilesSampler, SourceSSProfilesTextureSizeAndInvSize, InSampleIndex, InSubsurfaceProfileInt);
}

bool IsSubsurfaceProfileUseBurley()
{
	// 0..255, which SubSurface profile to pick
	float Type = GetSourceSubsurfaceProfileTexture(SSSS_BOUNDARY_COLOR_BLEED_OFFSET, SourceSubsurfaceProfileInt).a;

	return abs(Type - SSS_TYPE_BURLEY) < 0.01f;
}

float3 BurleyUnit(float Angle, float3 Sharpness)
{
	return exp(Sharpness * Angle);
}

// The integral of cos(x) * e^(a * x)
float3 IntegralCosineWithExponent(float Angle, float SinAngle, float CosAngle, float3 Sharpness)
{
	return (SinAngle + Sharpness * CosAngle) * BurleyUnit(Angle, Sharpness) / (1 + Sharpness * Sharpness);
}

// The integral of sin(x) * e^(a * x)
float3 IntegralSineWithExponent(float Angle, float SinAngle, float CosAngle, float3 Sharpness)
{
	return (-CosAngle + Sharpness * SinAngle) * BurleyUnit(Angle, Sharpness) / (1 + Sharpness * Sharpness);
}

// Theta is the angle between N and L, BurleyA is -((s * r) / m) * 2 / PI, BurleyB is BurleyA / 3
float3 ApproximationRingIntegralOfBurleyDiffusion(float Angle, float SinAngle, float CosAngle, float Theta, float CosTheta, float SinTheta, float3 BurleyA, float3 BurleyB)
{
	return CosTheta * (IntegralCosineWithExponent(Angle, SinAngle, CosAngle, BurleyA) + IntegralCosineWithExponent(Angle, SinAngle, CosAngle, BurleyB))
			-SinTheta * (IntegralSineWithExponent(Angle, SinAngle, CosAngle, BurleyA) + IntegralSineWithExponent(Angle, SinAngle, CosAngle, BurleyB));
}

float3 IntegralBurleyUnit(float Angle, float3 Sharpness)
{
	return (1 / Sharpness) * BurleyUnit(Angle, Sharpness);
}

float3 IntegralBurleyDiffusion(float Angle, float3 BurleyA, float3 BurleyB)
{
	return IntegralBurleyUnit(Angle, BurleyA) + IntegralBurleyUnit(Angle, BurleyB);
}

float4 GetIrradianceFromBurleyDiffusionOnRing(float4 SurfaceAlbedo, float4 DiffuseMeanFreePath, float Curvature, float Theta, float CosTheta, float SinTheta)
{
	const float HalfPI = PI * 0.5f;

	float3 S3D = GetScalingFactor3D(SurfaceAlbedo.xyz);

	// d = DiffuseMeanFreePath / S;
	// r = 1.0f / Curvature;
	// d / r = DiffuseMeanFreePath.xyz * Curvature / S3D;
	// Multiply 7.0f / 3.0f / PI to be close to the range of 2 * sin(x/2);
	float3 BurleyA = (-1.0f * S3D / (DiffuseMeanFreePath.xyz * Curvature)) * 2.0f / PI;

	float3 BurleyB = BurleyA / 3.0f;

	// Integral on the ring (from -PI to PI) to normalize the final integral.
	float3 IntegralOnRing = 2 * (IntegralBurleyDiffusion(PI, BurleyA, BurleyB) - IntegralBurleyDiffusion(0.0f, BurleyA, BurleyB));

	float3 IntegralOnNegativeAngle, IntegralOnPositiveAngle, BurleyDiffuse;
	if (Theta <= HalfPI)
	{
		// Integral from -(PI/2 + Theta) to 0.
		float AngleStart = -(HalfPI + Theta);
		IntegralOnNegativeAngle = ApproximationRingIntegralOfBurleyDiffusion(0.0f, 0.0f, 1.0f, Theta, CosTheta, SinTheta, -BurleyA, -BurleyB) - ApproximationRingIntegralOfBurleyDiffusion(AngleStart, sin(AngleStart), cos(AngleStart), Theta, CosTheta, SinTheta, -BurleyA, -BurleyB);

		// Integral from 0 to PI/2 - Theta
		float AngleEnd = HalfPI - Theta;
		IntegralOnPositiveAngle = ApproximationRingIntegralOfBurleyDiffusion(AngleEnd, sin(AngleEnd), cos(AngleEnd), Theta, CosTheta, SinTheta, BurleyA, BurleyB) - ApproximationRingIntegralOfBurleyDiffusion(0.0f, 0.0f, 1.0f, Theta, CosTheta, SinTheta, BurleyA, BurleyB);

	}
	else
	{
		// Integral from -PI to PI/2 - Theta.
		float AngleEnd = HalfPI - Theta; 
		IntegralOnNegativeAngle = ApproximationRingIntegralOfBurleyDiffusion(AngleEnd, sin(AngleEnd), cos(AngleEnd), Theta, CosTheta, SinTheta, -BurleyA, -BurleyB) - ApproximationRingIntegralOfBurleyDiffusion(-PI, 0.0f, -1.0f, Theta, CosTheta, SinTheta, -BurleyA, -BurleyB);

		// Integral from 3*PI/2 - Theta to PI
		float AngleStart = HalfPI * 3 - Theta;
		IntegralOnPositiveAngle = ApproximationRingIntegralOfBurleyDiffusion(PI, 0.0f, -1.0f, Theta, CosTheta, SinTheta, BurleyA, BurleyB) - ApproximationRingIntegralOfBurleyDiffusion(AngleStart, sin(AngleStart), cos(AngleStart), Theta, CosTheta, SinTheta, BurleyA, BurleyB);

	}

	BurleyDiffuse = (IntegralOnNegativeAngle + IntegralOnPositiveAngle) / IntegralOnRing;

//	float3 IntegralOnPI = ApproximationRingIntegralOfBurleyDiffusion(PI, 0.0f, -1.0f, 0.0f, 1.0f, 0.0f, BurleyA, BurleyB);

//	float3 ScatteringFactor3D = 2.0f * (ApproximationRingIntegralOfBurleyDiffusion(PI, 0.0f, -1.0f, 0.0f, 1.0f, 0.0f, BurleyA, BurleyB) - IntegralOnPI) / IntegralOnRing;

//	float ScatteringFactor = max3(ScatteringFactor3D.x, ScatteringFactor3D.y, ScatteringFactor3D.z);

	return float4(BurleyDiffuse, 1.0f);
}

void SSProfilePreIntegratedPS(
	in float4 SvPosition : SV_POSITION,
	out HALF4_TYPE OutColor : SV_Target0)
{
	OutColor = HALF4_TYPE(1.0f, 1.0f, 1.0f, 0.0f);

	float2 UVPos = SvPosition.xy * TargetSSProfilesPreIntegratedTextureSizeAndInvSize.zw;

	float UnClampedNoL = UVPos.x * 2.0f - 1.0f;
	float Curvature = UVPos.y;

	uint SurfaceAlbedoSampleIndex = BSSS_SURFACEALBEDO_OFFSET;
	uint DiffuseMeanFreePathSampleIndex = BSSS_DMFP_OFFSET;

	float4 SurfaceAlbedo = GetSourceSubsurfaceProfileTexture(SurfaceAlbedoSampleIndex, SourceSubsurfaceProfileInt);
	float WorldUnitScale = DecodeWorldUnitScale(GetSourceSubsurfaceProfileTexture(SSSS_TINT_SCALE_OFFSET, SourceSubsurfaceProfileInt).a) * BURLEY_CM_2_MM;
	float4 DiffuseMeanFreePath = max(DecodeDiffuseMeanFreePath(GetSourceSubsurfaceProfileTexture(DiffuseMeanFreePathSampleIndex, SourceSubsurfaceProfileInt)) * WorldUnitScale, 0.01f);;

	float CosTheta = UnClampedNoL;
	float Theta = acos(CosTheta);
	float SinTheta = sin(Theta);

	OutColor = GetIrradianceFromBurleyDiffusionOnRing(SurfaceAlbedo, DiffuseMeanFreePath, Curvature, Theta, CosTheta, SinTheta);
}