// Copyright Epic Games, Inc. All Rights Reserved.

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)

//------------------------------------------------------- ENUM VALUES

#define CASTING_PASS_STANDALONE 0
#define CASTING_PASS_PROBE_OCCLUSION 1

#define DIFFUSE_TERM 0
#define SPECULAR_TERM 1


//------------------------------------------------------- CONFIGS

#define DEBUG_SSRT 0
#define DEBUG_RAY_COUNT 0


#define CONFIG_LDS_STORE_VIEW_N 1

#if QUALITY == 1
	#define CONFIG_RAY_STEPS 8
	#define CONFIG_RAY_COUNT 4

#elif QUALITY == 2
	#define CONFIG_RAY_STEPS 8
	#define CONFIG_RAY_COUNT 8

#elif QUALITY == 3
	#define CONFIG_RAY_STEPS 8
	#define CONFIG_RAY_COUNT 16

#elif QUALITY == 4
	#define CONFIG_RAY_STEPS 12
	#define CONFIG_RAY_COUNT 32

#else
	#error Unknown Quality.
#endif

#if CONFIG_RAY_COUNT == 4
	#define TILE_PIXEL_SIZE_X 8
	#define TILE_PIXEL_SIZE_X_LOG 3

	#define TILE_PIXEL_SIZE_Y 8
	#define TILE_PIXEL_SIZE_Y_LOG 3

#elif CONFIG_RAY_COUNT == 8
	#define TILE_PIXEL_SIZE_X 8
	#define TILE_PIXEL_SIZE_X_LOG 3

	#define TILE_PIXEL_SIZE_Y 4
	#define TILE_PIXEL_SIZE_Y_LOG 2

#elif CONFIG_RAY_COUNT == 16
	#define TILE_PIXEL_SIZE_X 4
	#define TILE_PIXEL_SIZE_X_LOG 2

	#define TILE_PIXEL_SIZE_Y 4
	#define TILE_PIXEL_SIZE_Y_LOG 2

#elif CONFIG_RAY_COUNT == 32
	#define TILE_PIXEL_SIZE_X 4
	#define TILE_PIXEL_SIZE_X_LOG 2

	#define TILE_PIXEL_SIZE_Y 2
	#define TILE_PIXEL_SIZE_Y_LOG 1

#else
	#error Unknown Quality.
#endif


#if CONFIG_RAY_COUNT > 1
#define CONFIG_KARIS_WEIGHTING 1
#else
#define CONFIG_KARIS_WEIGHTING 0
#endif


//------------------------------------------------------- CONFIG DISABLED DEFAULTS

#ifndef CONFIG_LDS_STORE_VIEW_N
	#define CONFIG_LDS_STORE_VIEW_N 0
#endif




#define TILE_PIXEL_COUNT (TILE_PIXEL_SIZE_X * TILE_PIXEL_SIZE_Y)
#define LANE_PER_GROUPS (TILE_PIXEL_COUNT * CONFIG_RAY_COUNT)
		

//------------------------------------------------------- INCLUDES

#if DIM_LIGHTING_TERM == DIFFUSE_TERM
#define IS_SSGI_SHADER 1
#endif

#include "SSRTRayCast.ush"
#include "SSRTTileClassificationBuffer.ush"
#include "../DeferredShadingCommon.ush"
#include "../Random.ush"
#include "../BRDF.ush"
#include "../MonteCarlo.ush"
#include "../SceneTextureParameters.ush"
#include "../Substrate/Substrate.ush"
#include "../HZB.ush"

//------------------------------------------------------- PARAMETERS

uint bRejectUncertainRays;

Texture2D<float> ProbeOcclusionDistanceTexture;

Texture2D ColorTexture;

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define ColorTextureSampler GlobalPointClampedSampler
#else
SamplerState ColorTextureSampler;
#endif

float4 ColorBufferScaleBias;
float2 ReducedColorUVMax;

float PixelPositionToFullResPixel;
float2 FullResPixelOffset;

RWTexture2D<float4>	IndirectDiffuseOutput;
RWTexture2D<float>	AmbientOcclusionOutput;

RWTexture2D<float4> DebugOutput;


//------------------------------------------------------- LDS

groupshared uint SharedMemory0[LANE_PER_GROUPS * 2];
groupshared uint SharedMemory1[LANE_PER_GROUPS];


//------------------------------------------------------- FUNCTIONS

#if LANE_PER_GROUPS == 64 && 0
	#undef GroupMemoryBarrierWithGroupSync
	#define GroupMemoryBarrierWithGroupSync()
#endif

uint CompressN(float3 N)
{
	// matches 8bits GBuffer A to be lossless.
	uint3 K = uint3(saturate(N * 0.5 + 0.5) * 255.0);
	return uint(K.x << 0 | K.y << 8 | K.z << 16);
}

float3 DecompressN(uint EncodedN)
{
	uint3 K;
	K.x = (EncodedN >>  0) & 0xFF;
	K.y = (EncodedN >>  8) & 0xFF;
	K.z = (EncodedN >> 16) & 0xFF;

	return float3(K) * (2.0 / 255.0) - 1.0;
}

uint2 DecodeGroupPixelOffset(uint GroupPixelId)
{
    return uint2(GroupPixelId % TILE_PIXEL_SIZE_X, (GroupPixelId >> TILE_PIXEL_SIZE_X_LOG) % TILE_PIXEL_SIZE_Y);
}

uint EncodeGroupPixelOffset(uint2 GroupPixelOffset)
{
	return GroupPixelOffset.x | (GroupPixelOffset.y << TILE_PIXEL_SIZE_X_LOG);
}

uint2 ComputePixelPosition(uint2 GroupId, uint2 GroupPixelOffset)
{
    return GroupId * uint2(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y) + GroupPixelOffset;
}

void UpdateLane2DCoordinateInformations(
	uint2 PixelPosition,
	inout float2 BufferUV,
	inout float2 ScreenPos,
	inout float2 FullResPixelPosition)
{
	FullResPixelPosition = PixelPosition * PixelPositionToFullResPixel + FullResPixelOffset;

	// TODO: split screen
	BufferUV = FullResPixelPosition * View.BufferSizeAndInvSize.zw;
    ScreenPos = ViewportUVToScreenPos(FullResPixelPosition * View.ViewSizeAndInvSize.zw);
}

float3 ComputeTranslatedWorldPositions(float2 ScreenPos, float SceneDepth)
{
	return mul(float4(GetScreenPositionForProjectionType(ScreenPos, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
}

uint2 ComputePixelPosition(float3 TranslatedWorldPosition)
{
	float4 ClipPosition = mul(float4(TranslatedWorldPosition, 1), View.TranslatedWorldToClip);
	float2 ScreenPos = ClipPosition.xy * rcp(ClipPosition.w);
	float2 ViewportUV = ScreenPosToViewportUV(ScreenPos);
	return uint2(ViewportUV * View.ViewSizeAndInvSize.xy);
}

float ComputeSceneDepth(float3 TranslatedWorldPosition)
{
	// TODO: do everything in view space instead of world space?
	return mul(float4(TranslatedWorldPosition, 1.0), View.TranslatedWorldToView).z;
}

uint2 ComputeRandomSeed(uint2 PixelPosition)
{
	return Rand3DPCG16(int3(PixelPosition, View.StateFrameIndexMod8)).xy;
}

float3 ComputeL(float3 N, float2 E)
{
	float3x3 TangentBasis = GetTangentBasis(N);
#if 1	
	float3 TangentL = CosineSampleHemisphereConcentric(E).xyz;
#else
	float3 TangentL = CosineSampleHemisphere(E).xyz;
#endif
	return mul(TangentL, TangentBasis);
}


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_PIXEL_SIZE_X, TILE_PIXEL_SIZE_Y, CONFIG_RAY_COUNT)]
void MainCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	// Id of the wave in the group.
	uint GroupWaveIndex = GroupThreadIndex / 64;
	
	FSSRTTileInfos TileInfos;
	{
		const uint BinsAddress = TILE_PIXEL_COUNT * 2;

		uint GroupPixelId = GroupThreadIndex % TILE_PIXEL_COUNT;
		uint RaySequenceId = GroupThreadIndex / TILE_PIXEL_COUNT;

		// Compute TileCoord from GroupId to ensure the compiler understand it is group invariant to use scalar load.
		uint2 TileCoord = GroupId / uint2(SSRT_TILE_RES_DIVISOR / TILE_PIXEL_SIZE_X, SSRT_TILE_RES_DIVISOR / TILE_PIXEL_SIZE_Y);
		TileInfos = LoadTileInfos(TileCoord);

		// Store GBuffer into LDS
		BRANCH
		if (RaySequenceId == 0)
		{
			uint2 GroupPixelOffset = DecodeGroupPixelOffset(GroupPixelId);
            uint2 PixelPosition = ComputePixelPosition(GroupId, GroupPixelOffset);
				
			float2 BufferUV = 0;
			float2 ScreenPos = 0;
			float2 FullResPixelPosition = 0;
			UpdateLane2DCoordinateInformations(PixelPosition, /* out */ BufferUV, /* out */ ScreenPos, /* out */ FullResPixelPosition);
	
		#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(FullResPixelPosition, 0)));
			const float Roughness = TopLayerData.Roughness;
			const float3 WorldNormal = TopLayerData.WorldNormal;
			const bool bIsValid = IsSubstrateMaterial(TopLayerData);
		#else // SUBSTRATE_GBUFFER_FORMAT==1
			FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);
			const float Roughness = GBuffer.Roughness;
			const float3 WorldNormal = GBuffer.WorldNormal;
			const bool bIsValid = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
		#endif // SUBSTRATE_GBUFFER_FORMAT==1

			float DeviceZ = SceneDepthTexture.SampleLevel(SceneDepthTextureSampler, BufferUV, 0).r;

			float ProbeOcclusionDistance = ProbeOcclusionDistanceTexture.Load(int3(PixelPosition, 0)).x;

			bool bTraceRay = bIsValid;
				
			#if CONFIG_LDS_STORE_VIEW_N
				SharedMemory0[TILE_PIXEL_COUNT * 0 | GroupPixelId] = CompressN(mul(float4(WorldNormal, 0), View.TranslatedWorldToView).xyz);
			#else
				SharedMemory0[TILE_PIXEL_COUNT * 0 | GroupPixelId] = CompressN(WorldNormal);
			#endif
			SharedMemory0[TILE_PIXEL_COUNT * 1 | GroupPixelId] = asuint(bTraceRay ? DeviceZ : -1.0);

			#if DIM_LIGHTING_TERM == DIFFUSE_TERM
			{
				// NOP
			}
			#elif DIM_LIGHTING_TERM == SPECULAR_TERM
			{
				SharedMemory1[GroupPixelId] = asuint(Roughness);
			}
			#else
				#error Unnimplemented
			#endif
		}
		else if (GroupWaveIndex == 1) // TODO.
		{
			// Clears the bins
			SharedMemory0[BinsAddress | GroupPixelId] = 0;
		}
	}
	
	GroupMemoryBarrierWithGroupSync();
	
	// Shoot ray
	{
		uint GroupPixelId;
		uint RaySequenceId;
		uint CompressedN;
		float DeviceZ;
		float Roughness;
		float ProbeOcclusionDistance;
		bool bTraceRay;
		{
			GroupPixelId = GroupThreadIndex % TILE_PIXEL_COUNT;
			RaySequenceId = GroupThreadIndex / TILE_PIXEL_COUNT;

			uint Raw0 = SharedMemory0[TILE_PIXEL_COUNT * 0 | GroupPixelId];
			uint Raw1 = SharedMemory0[TILE_PIXEL_COUNT * 1 | GroupPixelId];
			uint Raw2 = SharedMemory1[TILE_PIXEL_COUNT * 0 | GroupPixelId];

			CompressedN = Raw0;
			DeviceZ = asfloat(Raw1);
			bTraceRay = asfloat(Raw1) > 0;

			#if DIM_LIGHTING_TERM == DIFFUSE_TERM
			{
				Roughness = 0;
				ProbeOcclusionDistance = 0;
			}
			#elif DIM_LIGHTING_TERM == SPECULAR_TERM
			{
				Roughness = asfloat(Raw2);
				ProbeOcclusionDistance = 0;
			}
			#else
				#error Unnimplemented
			#endif
		}

		GroupMemoryBarrierWithGroupSync();

		#if DEBUG_RAY_COUNT
			float DebugRayCount = 0.0;
		#endif
		uint2 CompressedColor;

		
		if (RaySequenceId == 0 && 0)
		{
			uint2 GroupPixelOffset = DecodeGroupPixelOffset(GroupPixelId);
			uint2 PixelPosition = ComputePixelPosition(GroupId, GroupPixelOffset);
			//DebugOutput[PixelPosition] = float4(DeviceZ, bTraceRay ? 1 : 0, 0, 0);
			//DebugOutput[PixelPosition] = float4(N * 0.5 + 0.5, 0);
		}

		BRANCH
		if (bTraceRay)
		{
			float3 N;
			float3 ViewN;
			#if CONFIG_LDS_STORE_VIEW_N
			{
				ViewN = DecompressN(CompressedN);
				N = mul(float4(ViewN, 0), View.ViewToTranslatedWorld).xyz;
			}
			#else
			{
				N = DecompressN(CompressedN);
				ViewN = mul(float4(N, 0), View.TranslatedWorldToView).xyz;
			}
			#endif

			float a = Roughness * Roughness;
			float a2 = a * a;

			float SceneDepth = ConvertFromDeviceZ(DeviceZ);

			uint2 GroupPixelOffset = DecodeGroupPixelOffset(GroupPixelId);
			uint2 PixelPosition = ComputePixelPosition(GroupId, GroupPixelOffset);
	
			float2 BufferUV = 0;
			float2 ScreenPos = 0;
			float2 FullResPixelPosition = 0;
			UpdateLane2DCoordinateInformations(PixelPosition, /* out */ BufferUV, /* out */ ScreenPos, /* out */ FullResPixelPosition); 
	
			float3 PositionTranslatedWorld = mul(float4(GetScreenPositionForProjectionType(ScreenPos, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
			float3 V = -GetCameraVectorFromTranslatedWorldPosition(PositionTranslatedWorld);

			float StepOffset = InterleavedGradientNoise(PixelPosition + 0.5, View.StateFrameIndexMod8);
	
			#if !SSGI_TRACE_CONE
				StepOffset -= 0.9;
			#endif

            bool bDebugPrint = all(PixelPosition == uint2(View.ViewSizeAndInvSize.xy) / 2);

			// Initialize the ray
			FSSRTRay Ray;
			float RayRoughness;
			float3 L;
			bool bRayWasClipped;

			#if DIM_LIGHTING_TERM == DIFFUSE_TERM
			{
				uint2 RandomSeed = ComputeRandomSeed(PixelPosition);
				float2 E = Hammersley16(RaySequenceId, CONFIG_RAY_COUNT, RandomSeed);		
			
				float3 ViewL = ComputeL(ViewN, E);
				Ray = InitScreenSpaceRay(ScreenPos, DeviceZ, ViewL);
				bRayWasClipped = true; // TODO(Guillaume)
				RayRoughness = 1.0;
				L = 0.0;
			}
			#elif DIM_LIGHTING_TERM == SPECULAR_TERM
			{
				uint2 RandomSeed = ComputeRandomSeed(PixelPosition);
				float2 E = Hammersley16(RaySequenceId, CONFIG_RAY_COUNT, RandomSeed);		
			
				float3x3 TangentBasis = GetTangentBasis(N);
				float3 TangentV = mul(TangentBasis, V);
				
				float3 H = mul(ImportanceSampleVisibleGGX(E, a, TangentV).xyz, TangentBasis);
				L = 2 * dot( V, H ) * H - V;

				Ray = InitScreenSpaceRayFromWorldSpace(
					PositionTranslatedWorld, L,
					/* WorldTMax = */ SceneDepth,
					/* SceneDepth = */ SceneDepth,
					/* SlopeCompareToleranceScale */ 2.0f,
					/* bExtendRayToScreenBorder = */ true,
					/* out */ bRayWasClipped);
				
				RayRoughness = Roughness * 0.25;
			}
			#else
				#error Unnimplemented
			#endif

			// Cast the ray
			float Level;
			float3 HitUVz;
			bool bHit;
			bool bUncertain;
			{
				float3 DebugOutput;
				CastScreenSpaceRay(
					FurthestHZBTexture, FurthestHZBTextureSampler,
					/* StartMipLevel = */ 1.0,
					CreateDefaultCastSettings(),
					Ray, RayRoughness, CONFIG_RAY_STEPS, StepOffset,
					HZBUvFactorAndInvFactor, bDebugPrint,
					/* out */ DebugOutput,
					/* out */ HitUVz,
					/* out */ Level,
					/* out */ bHit,
					/* out */ bUncertain);
				
				#if DEBUG_RAY_COUNT
					DebugRayCount += 1.0;
				#endif
			}

			// Ray is also uncertain if it has been clipped.
			bUncertain = bUncertain || bRayWasClipped;

			#if 0 // Backface check
			if (bHit)
			{
				float3 SampleNormal = GetGBufferDataFromSceneTextures(HitUVz.xy).WorldNormal;
				bHit = dot(SampleNormal, L) < 0;
			}
			#endif
			
			// if there was a hit
			BRANCH
			if (bHit)
			{
				float2 ReducedColorUV =  HitUVz.xy * ColorBufferScaleBias.xy + ColorBufferScaleBias.zw;
				ReducedColorUV = min(ReducedColorUV, ReducedColorUVMax);

				float4 SampleColor = ColorTexture.SampleLevel(ColorTextureSampler, ReducedColorUV, Level);
				//SampleColor = float4(ReducedColorUV.xy, 0, 1);

				float SampleColorWeight = 1.0;
				#if CONFIG_KARIS_WEIGHTING
					SampleColorWeight *= rcp( 1 + Luminance(SampleColor.rgb) );
				#endif
				
				float3 DiffuseColor = SampleColor.rgb * SampleColorWeight;
				float AmbientOcclusion = 1.0;
				
				#if CONFIG_COLOR_TILE_CLASSIFICATION
				{
					float Lumi = Luminance(DiffuseColor.rgb);
					AmbientOcclusion *= saturate(Lumi / 0.25);
				}
				#endif

				CompressedColor.x = asuint(f32tof16(DiffuseColor.r) << 16 | f32tof16(DiffuseColor.g));
				CompressedColor.y = asuint(f32tof16(DiffuseColor.b) << 16 | f32tof16(AmbientOcclusion));
			}
			else
			{
				CompressedColor = uint2(0, 0);
			}
			
		}
		else // if (!bTraceRay)
        {
            CompressedColor = uint2(0, 0);
        }
		
		// Output debugging info instead of actual intersection.
		#if DEBUG_RAY_COUNT
		{
			CompressedColor.x = asuint(f32tof16(DebugRayCount) << 16);
			CompressedColor.y = 0;
		}
		#endif
		
		uint DestPos = GroupPixelId + RaySequenceId * TILE_PIXEL_COUNT;
			
		SharedMemory0[LANE_PER_GROUPS * 0 | DestPos] = CompressedColor.x;
		SharedMemory0[LANE_PER_GROUPS * 1 | DestPos] = CompressedColor.y;
    }
	
	GroupMemoryBarrierWithGroupSync();
	
	// Store ray to UAV
	BRANCH
	if (GroupThreadIndex < TILE_PIXEL_COUNT)
	{
		const uint GroupPixelId = GroupThreadIndex;
	
		float3 DiffuseColor = 0;
		float AmbientOcclusion = 0;
		uint SampleMask = 0;

		UNROLL_N(CONFIG_RAY_COUNT)
		for (uint RaySequenceId = 0; RaySequenceId < CONFIG_RAY_COUNT; RaySequenceId++)
		{
			uint SrcPos = GroupPixelId + RaySequenceId * TILE_PIXEL_COUNT;

			uint Row0 = SharedMemory0[LANE_PER_GROUPS * 0 | SrcPos];
			uint Row1 = SharedMemory0[LANE_PER_GROUPS * 1 | SrcPos];

			DiffuseColor.r += f16tof32(Row0 >> 16);
			DiffuseColor.g += f16tof32(Row0 >>  0);
			DiffuseColor.b += f16tof32(Row1 >> 16);
			AmbientOcclusion += f16tof32(Row1 >> 0);

			SampleMask |= (f16tof32(Row1 >> 0) > 0.0) ? (1u << RaySequenceId) : 0;
		}

		#if CONFIG_RAY_COUNT > 1
		{
			DiffuseColor *= rcp(float(CONFIG_RAY_COUNT));
			AmbientOcclusion *= rcp(float(CONFIG_RAY_COUNT));
		}	
		#endif
		
		#if CONFIG_KARIS_WEIGHTING
		{
			DiffuseColor *= rcp( 1 - Luminance(DiffuseColor) );
		}	
		#endif

		// TODO Guillaume: need to have engine wide consistent solution for IndirectLightingColorScale.
		//DiffuseColor *= View.IndirectLightingColorScale;
		AmbientOcclusion = 1 - AmbientOcclusion;

		uint2 GroupPixelOffset = DecodeGroupPixelOffset(GroupPixelId);
		uint2 OutputPixelCoordinate = ComputePixelPosition(GroupId, GroupPixelOffset);

		// Output.
		IndirectDiffuseOutput[OutputPixelCoordinate] = float4(DiffuseColor, 1.0);
		AmbientOcclusionOutput[OutputPixelCoordinate] = AmbientOcclusion;
	} // if (GroupThreadIndex < TILE_PIXEL_COUNT)
} // MainCS()
