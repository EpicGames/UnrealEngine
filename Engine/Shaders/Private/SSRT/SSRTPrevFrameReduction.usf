// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_COLOR_TILE_CLASSIFICATION 0

#include "SSRTRayCast.ush"
#include "../SceneTextureParameters.ush"
#include "../HZB.ush"

#define GROUP_TILE_SIZE 8
#define GROUP_PIXEL_COUNT GROUP_TILE_SIZE * GROUP_TILE_SIZE


Texture2D PrevSceneColor;
Texture2D PrevSceneDepth;

Texture2D HigherMipTexture;
Texture2D HigherAlphaMipTexture;

#if SUPPORTS_INDEPENDENT_SAMPLERS
#if DIM_LEAK_FREE
#define PrevSceneColorSampler GlobalPointClampedSampler
#else
#define PrevSceneColorSampler GlobalBilinearClampedSampler
#endif
#define PrevSceneDepthSampler GlobalPointClampedSampler
#define HigherMipTextureSampler GlobalPointClampedSampler
#define HigherAlphaMipTextureSampler GlobalPointClampedSampler
#else
SamplerState PrevSceneColorSampler;
SamplerState PrevSceneDepthSampler;
SamplerState HigherMipTextureSampler;
SamplerState HigherAlphaMipTextureSampler;
#endif

float4 PrevBufferBilinearUVMinMax;
float4 PrevScreenPositionScaleBias;
float2 ReducedSceneColorSize;
float2 ReducedSceneColorTexelSize;
float2 HigherMipBufferBilinearMax;
float PrevSceneColorPreExposureCorrection;
float MinimumLuminance;
float HigherMipDownScaleFactor;
float SkyDistance;

RWTexture2D<float4>	ReducedSceneColorOutput_0;
RWTexture2D<float4>	ReducedSceneColorOutput_1;
RWTexture2D<float4>	ReducedSceneColorOutput_2;

RWTexture2D<float>	ReducedSceneAlphaOutput_0;
RWTexture2D<float>	ReducedSceneAlphaOutput_1;
RWTexture2D<float>	ReducedSceneAlphaOutput_2;

groupshared uint SharedMemory[GROUP_PIXEL_COUNT * 2];

#if DIM_LEAK_FREE
groupshared float SharedFurthestDepth[GROUP_PIXEL_COUNT];
#endif


float3 ComputeTranslatedWorldPosition(float2 ScreenPosition, float WorldDepth, bool bIsPrevFrame)
{
	float4 ClipPosition = float4(GetScreenPositionForProjectionType(ScreenPosition, WorldDepth), WorldDepth, 1);

	if (bIsPrevFrame)
	{
		float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
		return mul(ClipPosition, View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;
	}
	
	return mul(ClipPosition, View.ScreenToTranslatedWorld).xyz;
}


[numthreads(GROUP_TILE_SIZE, GROUP_TILE_SIZE, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	#if DIM_LOWER_MIPS
		float2 SceneBufferUV = HigherMipDownScaleFactor * (2.0 * float2(DispatchThreadId) + 1.0) * View.BufferSizeAndInvSize.zw;
	#else
		float2 SceneBufferUV = (float2(DispatchThreadId) + 0.5) * View.BufferSizeAndInvSize.zw;
	#endif

	SceneBufferUV = clamp(SceneBufferUV, View.BufferBilinearUVMinMax.xy, View.BufferBilinearUVMinMax.zw);
	float2 ViewportUV = BufferUVToViewportUV(SceneBufferUV);
    float2 ScreenPosition = ViewportUVToScreenPos(ViewportUV);
	
	float4 PrevColor;
	float WorldDepth;

	#if DIM_LOWER_MIPS
	{
		#if DIM_LEAK_FREE
		{
			{
				float HZBDeviceZ = FurthestHZBTexture.SampleLevel(FurthestHZBTextureSampler, ViewportUV * ViewportUVToHZBBufferUV, 2.0).r;
				WorldDepth = ConvertFromDeviceZ(HZBDeviceZ);
			}

			float WorldBluringRadius = GetDepthPixelRadiusForProjectionType(WorldDepth) * 100.0f;
			float InvSquareWorldBluringRadius = rcp(WorldBluringRadius * WorldBluringRadius);

			PrevColor = 0.0;

			UNROLL_N(4)
			for (uint i = 0; i < 4; i++)
			{
				const float2 TexelOffset = float2(i % 2, i / 2) - 0.5;

				float2 HZBBufferUV = (ViewportUV + TexelOffset * HigherMipDownScaleFactor * View.ViewSizeAndInvSize.zw) * ViewportUVToHZBBufferUV;
				float SampleDeviceZ = FurthestHZBTexture.SampleLevel(FurthestHZBTextureSampler, HZBBufferUV, 1.0).r;

				float SampleDist = WorldDepth - ConvertFromDeviceZ(SampleDeviceZ);

				float SampleWeight = 0.25 * saturate(1 - SampleDist * SampleDist * InvSquareWorldBluringRadius);

				float2 SampleUV = HigherMipDownScaleFactor * (2.0 * float2(DispatchThreadId) + 1.0 + TexelOffset) * 0.5 * ReducedSceneColorTexelSize;
				SampleUV = min(SampleUV, HigherMipBufferBilinearMax);

				float4 SampleColor = float4(
					HigherMipTexture.SampleLevel(HigherMipTextureSampler, SampleUV, 0).rgb,
					HigherAlphaMipTexture.SampleLevel(HigherAlphaMipTextureSampler, SampleUV, 0).r);

				PrevColor += SampleColor * SampleWeight;
			}
		}
		#else
		{
			float2 HigherMipUV = HigherMipDownScaleFactor * (float2(DispatchThreadId) * 1.0 + 0.5) * ReducedSceneColorTexelSize;
			PrevColor = float4(HigherMipTexture.SampleLevel(HigherMipTextureSampler, HigherMipUV, 0).rgb, 1);
		}
		#endif
	}
	#else
	{
		float DeviceZ = SampleDeviceZFromSceneTextures(SceneBufferUV);
		WorldDepth = ConvertFromDeviceZ(DeviceZ);

		// Camera motion for pixel (in ScreenPos space).
		float4 ThisClip = float4(ScreenPosition, DeviceZ, 1);
		float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
		float2 PrevScreen = PrevClip.xy / PrevClip.w;

		bool bIsSky = WorldDepth > SkyDistance && SkyDistance > 0.0;

		float4 EncodedVelocity = GBufferVelocityTexture.SampleLevel(GBufferVelocityTextureSampler, SceneBufferUV, 0);
		if (EncodedVelocity.x > 0.0)
		{
			PrevScreen = ThisClip.xy - DecodeVelocityFromTexture(EncodedVelocity).xy;
		}

		float2 PrevFrameUV = PrevScreen.xy * PrevScreenPositionScaleBias.xy + PrevScreenPositionScaleBias.zw;

		// Fetch color.
		#if DIM_LEAK_FREE
		{
			float3 RefWorldPosition = ComputeTranslatedWorldPosition(ScreenPosition, WorldDepth, /* bIsPrevFrame = */ false);

			float NoV = dot(View.TranslatedWorldCameraOrigin - normalize(RefWorldPosition), GetGBufferDataFromSceneTextures(SceneBufferUV).WorldNormal);
				
			float WorldBluringRadius = GetDepthPixelRadiusForProjectionType(WorldDepth) * 100.0f;
			float InvSquareWorldBluringRadius = rcp(WorldBluringRadius * WorldBluringRadius);

			//PrevColor = 0.0;

			//UNROLL_N(4)
			//for (uint i = 0; i < 4; i++)
			{
				float2 SampleUV = PrevFrameUV; // + View.BufferSizeAndInvSize.zw * (float2(i % 2, i / 2) - 0.5);
				SampleUV = clamp(SampleUV, PrevBufferBilinearUVMinMax.xy, PrevBufferBilinearUVMinMax.zw);

				float PrevDeviceZ = PrevSceneDepth.SampleLevel(PrevSceneDepthSampler, SampleUV, 0).r;

				float3 SampleWorldPosition = ComputeTranslatedWorldPosition(PrevScreen.xy, ConvertFromDeviceZ(PrevDeviceZ), /* bIsPrevFrame = */ true);

				float SampleDistSquare = length2(RefWorldPosition - SampleWorldPosition);

				float SampleWeight = saturate(1 - SampleDistSquare * InvSquareWorldBluringRadius);

				PrevColor = float4(PrevSceneColor.SampleLevel(PrevSceneColorSampler, SampleUV, 0).rgb * SampleWeight, SampleWeight);
			}
		}
		#else
		{
			PrevColor = float4(PrevSceneColor.SampleLevel(PrevSceneColorSampler, PrevFrameUV, 0).rgb, 1.0);
		}
		#endif

		PrevColor = -min(-PrevColor, 0.0);


		// Ignore sky contribution because in skylight
		if (bIsSky)
		{
			PrevColor = 0;
		}

		// Correct scene color exposure.
		PrevColor.rgb *= PrevSceneColorPreExposureCorrection;

		// Apply vignette to the color.
		{
			float Vignette = min(ComputeHitVignetteFromScreenPos(ScreenPosition), ComputeHitVignetteFromScreenPos(PrevScreen));
			PrevColor *= Vignette;
		}

		// Apply luminance to mask in only significant lighting worth tracing rays towards.
		#if CONFIG_COLOR_TILE_CLASSIFICATION
		if (MinimumLuminance > 0.0)
		{
			float ColorLuminance = Luminance(PrevColor.rgb);

			PrevColor.rgb *= saturate((ColorLuminance - MinimumLuminance) / MinimumLuminance);
		}
		#endif
	}
	#endif

	// Output mip 0
	#if DIM_LOWER_MIPS
	{
		ReducedSceneColorOutput_0[DispatchThreadId] = float4(PrevColor.rgb, 0);

		#if DIM_LEAK_FREE
			ReducedSceneAlphaOutput_0[DispatchThreadId] = PrevColor.a;
		#endif
	}
	#endif

	// Reduce lower mip levels.
	{
		// Store to LDS
		{
			SharedMemory[GROUP_PIXEL_COUNT * 0 | GroupThreadIndex] = (f32tof16(PrevColor.r) << 0) | (f32tof16(PrevColor.g) << 16);
			SharedMemory[GROUP_PIXEL_COUNT * 1 | GroupThreadIndex] = (f32tof16(PrevColor.b) << 0) | (f32tof16(PrevColor.a) << 16);

			#if DIM_LEAK_FREE
				SharedFurthestDepth[GroupThreadIndex] = WorldDepth;
			#endif
		}
	
		GroupMemoryBarrierWithGroupSync();

		// Reduce lower mip levels.
		UNROLL
		for (uint MipLevel = 1; MipLevel < 3; MipLevel++)
		{
			const uint ReductionAmount = 1u << MipLevel;
			const uint NumberPixelInMip = GROUP_PIXEL_COUNT / (ReductionAmount * ReductionAmount);
			
			if (GroupThreadIndex < NumberPixelInMip)
            {
                uint2 OutputCoord = uint2(
					GroupThreadIndex % (GROUP_TILE_SIZE / ReductionAmount),
					GroupThreadIndex / (GROUP_TILE_SIZE / ReductionAmount));

				// Ray marchsing against FurthestHZB to avoid self intersections, so match here to remain conservative.
				#if DIM_LEAK_FREE
				float FurthestDepth;
				{
					UNROLL_N(2)
					for (uint x = 0; x < 2; x++)
					{
						UNROLL_N(2)
						for (uint y = 0; y < 2; y++)
						{
							uint2 Coord = OutputCoord * 2 + uint2(x, y);
							uint LDSIndex = Coord.x + Coord.y * ((2 * GROUP_TILE_SIZE) / ReductionAmount);

							float NeighborDepth = SharedFurthestDepth[LDSIndex];

							if (x == 0 && y == 0)
								FurthestDepth = NeighborDepth;
							else
								FurthestDepth = max(FurthestDepth, NeighborDepth);
						}
					}
				}

				float WorldBluringRadius = GetDepthPixelRadiusForProjectionType(FurthestDepth) * 100.0f;
				float InvSquareWorldBluringRadius = rcp(WorldBluringRadius * WorldBluringRadius);

				#endif

				float4 ReducedColor = 0;

				UNROLL
				for (uint x = 0; x < 2; x++)
				{
					UNROLL
					for (uint y = 0; y < 2; y++)
					{
						uint2 Coord = OutputCoord * 2 + uint2(x, y);
						uint LDSIndex = Coord.x + Coord.y * ((2 * GROUP_TILE_SIZE) / ReductionAmount);

						uint Raw0 = SharedMemory[GROUP_PIXEL_COUNT * 0 | LDSIndex];
						uint Raw1 = SharedMemory[GROUP_PIXEL_COUNT * 1 | LDSIndex];

						float4 Color;
						Color.r = f16tof32(Raw0 >> 0);
						Color.g = f16tof32(Raw0 >> 16);
						Color.b = f16tof32(Raw1 >> 0);
						Color.a = f16tof32(Raw1 >> 16);


						float SampleWeight = 1.0;
						#if DIM_LEAK_FREE
						{
							float NeighborDepth = SharedFurthestDepth[LDSIndex];
							float SampleDist = (FurthestDepth - NeighborDepth);

							SampleWeight = saturate(1 - (SampleDist * SampleDist) * InvSquareWorldBluringRadius);
						}
						#endif

						ReducedColor += Color * SampleWeight;
					}
                }

				ReducedColor *= rcp(4.0);

				uint2 OutputPosition = GroupId * (GROUP_TILE_SIZE / ReductionAmount) + OutputCoord;

				if (MipLevel == 1)
                {
					ReducedSceneColorOutput_1[OutputPosition] = float4(ReducedColor.rgb, 0);
					#if DIM_LEAK_FREE
						ReducedSceneAlphaOutput_1[OutputPosition] = ReducedColor.a;
					#endif
                }
				else if (MipLevel == 2)
                {
					ReducedSceneColorOutput_2[OutputPosition] = float4(ReducedColor.rgb, 0);
					#if DIM_LEAK_FREE
						ReducedSceneAlphaOutput_2[OutputPosition] = ReducedColor.a;
					#endif
                }
				
				SharedMemory[GROUP_PIXEL_COUNT * 0 | GroupThreadIndex] = (f32tof16(ReducedColor.r) << 0) | (f32tof16(ReducedColor.g) << 16);
				SharedMemory[GROUP_PIXEL_COUNT * 1 | GroupThreadIndex] = (f32tof16(ReducedColor.b) << 0) | (f32tof16(ReducedColor.a) << 16);

				#if DIM_LEAK_FREE
				{
					SharedFurthestDepth[GroupThreadIndex] = FurthestDepth;
				}
				#endif
            } // if (GroupThreadIndex < NumberPixelInMip)
        } // for (uint MipLevel = 1; MipLevel < 3; MipLevel++)
    }
} // MainCS()
