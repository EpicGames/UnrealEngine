// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Number of sample batched at same time.
#define SSRT_SAMPLE_BATCH_SIZE 4

#define SSRT_TILE_RES_DIVISOR_LOG2 3
#define SSRT_TILE_RES_DIVISOR 8
#define SSRT_DIRECTIONALITY_DIVISION 8

#ifndef IS_SSGI_SHADER
#define IS_SSGI_SHADER 0
#endif

#include "../Common.ush"	

#define SSGI_TRACE_CONE 0


#ifndef DEBUG_SSRT
	#define DEBUG_SSRT 0
#endif

#if DEBUG_SSRT

RWTexture2D<float4> ScreenSpaceRayTracingDebugOutput;


void PrintSample(
	float4 InHZBUvFactorAndInvFactor, 
	float2 HZBUV,
	float4 DebugColor)
{	
	HZBUV.xy *= InHZBUvFactorAndInvFactor.zw;

	uint2 Position = HZBUV.xy * View.ViewSizeAndInvSize.xy;

	ScreenSpaceRayTracingDebugOutput[Position] = DebugColor;
}

#endif


/** Return float multiplier to scale RayStepScreen such that it clip it right at the edge of the screen. */
float GetStepScreenFactorToClipAtScreenEdge(float2 RayStartScreen, float2 RayStepScreen)
{
	// Computes the scale down factor for RayStepScreen required to fit on the X and Y axis in order to clip it in the viewport
	const float RayStepScreenInvFactor = 0.5 * length(RayStepScreen);
	const float2 S = 1 - max(abs(RayStepScreen + RayStartScreen * RayStepScreenInvFactor) - RayStepScreenInvFactor, 0.0f) / abs(RayStepScreen);

	// Rescales RayStepScreen accordingly
	const float RayStepFactor = min(S.x, S.y) / RayStepScreenInvFactor;

	return RayStepFactor;
}


/** Structure that represent a ray to be shot in screen space. */
struct FSSRTRay
{
	float3 RayStartScreen;
	float3 RayStepScreen;

	float CompareTolerance;
};

/** Compile a ray for screen space ray casting. */
FSSRTRay InitScreenSpaceRayFromWorldSpace(
	float3 RayOriginTranslatedWorld,
	float3 WorldRayDirection,
	float WorldTMax,
	float SceneDepth,
	float SlopeCompareToleranceScale,
	const bool bExtendRayToScreenBorder,
	out bool bRayWasClipped,
	const float WorldTMaxClamp = 1000000)
{
	WorldTMax = min(WorldTMax, WorldTMaxClamp);

	float3 ViewRayDirection = mul(float4(WorldRayDirection, 0.0), View.TranslatedWorldToView).xyz;

	float RayEndWorldDistance = ViewRayDirection.z < 0.0 ? min(-0.95 * SceneDepth / ViewRayDirection.z, WorldTMax) : WorldTMax;

	float3 RayEndWorld = RayOriginTranslatedWorld + WorldRayDirection * RayEndWorldDistance;

	float4 RayStartClip = mul(float4(RayOriginTranslatedWorld, 1.0), View.TranslatedWorldToClip);
	float4 RayEndClip = mul(float4(RayEndWorld, 1.0), View.TranslatedWorldToClip);

	float3 RayStartScreen = RayStartClip.xyz * rcp(RayStartClip.w);
	float3 RayEndScreen = RayEndClip.xyz * rcp(RayEndClip.w);

	float4 RayDepthClip = RayStartClip + mul(float4(0, 0, RayEndWorldDistance, 0), View.ViewToClip);
	float3 RayDepthScreen = RayDepthClip.xyz * rcp(RayDepthClip.w);

	FSSRTRay Ray;
	Ray.RayStartScreen = RayStartScreen;
	Ray.RayStepScreen = RayEndScreen - RayStartScreen;

	float ClipToScreenFactor = GetStepScreenFactorToClipAtScreenEdge(RayStartScreen.xy, Ray.RayStepScreen.xy);
	if (!bExtendRayToScreenBorder)
	{
		bRayWasClipped = ClipToScreenFactor < 1.0 || RayEndWorldDistance != WorldTMax;
		ClipToScreenFactor = min(ClipToScreenFactor, 1.0);
	}
	else
	{
		bRayWasClipped = true;
	}

	Ray.RayStepScreen *= ClipToScreenFactor;
	if (IsOrthoProjection())
	{
		Ray.CompareTolerance = max(0, (RayStartScreen.z - RayDepthScreen.z) * SlopeCompareToleranceScale);
	}
	else
	{
		Ray.CompareTolerance = max(abs(Ray.RayStepScreen.z), (RayStartScreen.z - RayDepthScreen.z) * SlopeCompareToleranceScale);
	}
	return Ray;
} // InitScreenSpaceRayFromWorldSpace()

float4 ApplyProjMatrix(float4 V)
{
	return float4(
		V.xy * float2(View.ViewToClip[0][0],View.ViewToClip[1][1]),
		V.z * View.ViewToClip[2][2] + V.w * View.ViewToClip[3][2],
		V.z);
}

/** Compile a ray for screen space ray casting, but avoiding the computation of translated world position. */
// TODO: passdown ViewRayDirection instead?
FSSRTRay InitScreenSpaceRay(
	float2 ScreenPos,
	float DeviceZ,
	float3 ViewRayDirection)
{
	float3 RayStartScreen = float3(ScreenPos, DeviceZ);

	// float3 RayStartClip = float4(ScreenPos, DeviceZ, 1) * SceneDepth;
	// float4 RayEndClip = RayStartClip + mul(float4(WorldRayDirection * SceneDepth, 0), View.TranslatedWorldToClip);
	
	// float4 RayEndClip = mul(float4(WorldRayDirection, 0), View.TranslatedWorldToClip) + float4(RayStartScreen, 1); // * SceneDepth;
	float4 RayEndClip = ApplyProjMatrix(float4(ViewRayDirection, 0)) + float4(RayStartScreen, 1); // * SceneDepth;

	float3 RayEndScreen = RayEndClip.xyz * rcp(RayEndClip.w);

	// float3 RayStartClip = float4(ScreenPos, DeviceZ, 1) * SceneDepth;
	// float4 RayDepthClip = RayStartClip + mul(float4(0, 0, SceneDepth, 0), View.ViewToClip);
	// RayDepthClip.w = 2.0 * SceneDepth;
	// float3 RayDepthScreen = RayDepthClip.xyz * rcp(RayDepthClip.w);
	float3 RayDepthScreen = 0.5 * (RayStartScreen + mul(float4(0, 0, 1, 0), View.ViewToClip).xyz);

	FSSRTRay Ray;
	Ray.RayStartScreen = RayStartScreen;
	Ray.RayStepScreen = RayEndScreen - RayStartScreen;
	
	Ray.RayStepScreen *= GetStepScreenFactorToClipAtScreenEdge(RayStartScreen.xy, Ray.RayStepScreen.xy);

	// TODO
	#if IS_SSGI_SHADER
		Ray.CompareTolerance = max(abs(Ray.RayStepScreen.z), (RayStartScreen.z - RayDepthScreen.z) * 2);
	#else
		Ray.CompareTolerance = max(abs(Ray.RayStepScreen.z), (RayStartScreen.z - RayDepthScreen.z) * 4);
	#endif

	return Ray;
} // InitScreenSpaceRay()

/** Information about tiles in SGPRs. */
struct FSSRTTileInfos
{
	float Directionality[SSRT_DIRECTIONALITY_DIVISION];
	float ClosestDeviceZ;
	uint2 Coord;
	float _Padding[5];
};

/** Casting. */
struct FSSRTCastingSettings
{
	// (Compile time) whether should terminate the ray immediately if the ray become uncertain.
	bool bStopWhenUncertain;

	// (Compile time) whether should use tile informations
	bool bUseTileInfo;

	// (SGPRs) Information about the tile to speed up casting.
	FSSRTTileInfos TileInfos;
};


FSSRTCastingSettings CreateDefaultCastSettings()
{
	FSSRTCastingSettings Settings;
	Settings.bUseTileInfo = false;
	Settings.bStopWhenUncertain = false;
	Settings.TileInfos = (FSSRTTileInfos)0;
	return Settings;
}

uint ComputeRayDirectionId(FSSRTRay Ray)
{
	float2 RayStepPixel = Ray.RayStepScreen.xy * View.ViewSizeAndInvSize.xy;

	// Screen position is vertically flipped from 
	RayStepPixel.y = -RayStepPixel.y;

	// Compute screen ray direction in [0; 2 PI[
	float RayDirection = atan2(-RayStepPixel.y, -RayStepPixel.x) + PI;

	uint DirectionId;
#if SSRT_DIRECTIONALITY_DIVISION == 8
	FLATTEN
	if (RayDirection < (0.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION) || RayDirection >(7.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 0;
	}
	else if (RayDirection < (1.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 1;
	}
	else if (RayDirection < (2.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 2;
	}
	else if (RayDirection < (3.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 3;
	}
	else if (RayDirection < (4.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 4;
	}
	else if (RayDirection < (5.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 5;
	}
	else if (RayDirection < (6.5 * 2 * PI / SSRT_DIRECTIONALITY_DIVISION))
	{
		DirectionId = 6;
	}
	else
	{
		DirectionId = 7;
	}
#else
#error Need to update this code.
#endif

	return DirectionId;
}

/** Cast a screen space ray. */
void CastScreenSpaceRay(
	Texture2D Texture, SamplerState Sampler,
	float StartMipLevel,
	FSSRTCastingSettings CastSettings,
	FSSRTRay Ray,
	float Roughness,
	uint NumSteps, float StepOffset,
	float4 InHZBUvFactorAndInvFactor, 
	bool bDebugPrint,
	out float3 DebugOutput,
	out float3 OutHitUVz,
	out float Level,
	out bool bFoundHit,
	out bool bUncertain)
{
	DebugOutput = 0;

	const float3 RayStartScreen = Ray.RayStartScreen;
	const float3 RayStepScreen = Ray.RayStepScreen;

	float3 RayStartUVz = float3( (RayStartScreen.xy * float2( 0.5, -0.5 ) + 0.5) * InHZBUvFactorAndInvFactor.xy, RayStartScreen.z );
	float3 RayStepUVz  = float3(  RayStepScreen.xy  * float2( 0.5, -0.5 )		 * InHZBUvFactorAndInvFactor.xy, RayStepScreen.z );
	
	const float Step = 1.0 / NumSteps;
	float CompareTolerance = Ray.CompareTolerance * Step;
	
	float LastDiff = 0;
	Level = StartMipLevel;

	//StepOffset = View.GeneralPurposeTweak;

	RayStepUVz *= Step;
	float3 RayUVz = RayStartUVz + RayStepUVz * StepOffset;
	#if IS_SSGI_SHADER && SSGI_TRACE_CONE
		RayUVz = RayStartUVz;
	#endif
	
	#if DEBUG_SSRT
	{
		if (bDebugPrint)
			PrintSample(InHZBUvFactorAndInvFactor, RayStartUVz.xy, float4(1, 0, 0, 1));
	}
	#endif
	
	float4 MultipleSampleDepthDiff;
	bool4 bMultipleSampleHit; // TODO: Might consumes VGPRS if bug in compiler.
	
	#if IS_SSGI_SHADER && SSGI_TRACE_CONE
		const float ConeAngle = PI / 4;
		const float d = 1;
		const float r = d * sin(0.5 * ConeAngle);
		const float Exp = 1.6; //(d + r) / (d - r);
		const float ExpLog2 = log2(Exp);
		const float MaxPower = exp2(log2(Exp) * (NumSteps + 1.0)) - 0.9;

		{
			//Level = 2;
		}
	#endif

	// Compute the maximum of iteration.
	uint MaxIteration = NumSteps;
	if (CastSettings.bUseTileInfo)
	{
		uint DirectionId = ComputeRayDirectionId(Ray);

		float DirectionAndle = float(DirectionId) * (2 * PI * rcp(float(SSRT_DIRECTIONALITY_DIVISION)));

		float2 RayPixelDirection = float2(cos(DirectionAndle), sin(DirectionAndle));

		

		float2 cr = ScreenPosToViewportUV(RayStartScreen.xy) * View.ViewSizeAndInvSize.xy;
		float2 cre = ScreenPosToViewportUV(RayStartScreen.xy + RayStepScreen.xy) * View.ViewSizeAndInvSize.xy;

		// Ray equation (x - xr) * ar + dr;
		float xr = dot(cr, RayPixelDirection) * View.ViewSizeAndInvSize.z;
		float xre = dot(cre, RayPixelDirection) * View.ViewSizeAndInvSize.z;
		float dr = RayStartScreen.z;
		float ar = RayStepScreen.z * View.ViewSizeAndInvSize.x / dot(RayStepScreen.xy * float2(0.5, -0.5) * View.ViewSizeAndInvSize.xy, RayPixelDirection);

		float2 cc = CastSettings.TileInfos.Coord * SSRT_TILE_RES_DIVISOR + SSRT_TILE_RES_DIVISOR / 2;
		float2 c0 = cc - sign(RayPixelDirection) * float(SSRT_TILE_RES_DIVISOR / 2);
		float2 c1 = cc + sign(RayPixelDirection) * float(SSRT_TILE_RES_DIVISOR / 2);
		float x0 = dot(c0, RayPixelDirection) * View.ViewSizeAndInvSize.z;
		float x1 = dot(c1, RayPixelDirection) * View.ViewSizeAndInvSize.z;

		// Horizon equation is (x - xf) * af + df
		float af = CastSettings.TileInfos.Directionality[DirectionId];
		float df = CastSettings.TileInfos.ClosestDeviceZ;
		float xf = af > 0.0 ? x0 : x1;

		// Resolve x intersection of horizon and ray
		uint UsefulMaxIteration;
		{
			float adiff = ar - af;

			// Intersection of the ray and horizong
			FLATTEN
			if (ar > af)
			{
				float x = (df - dr + xr * ar - xf * af) / adiff;

				// Compute the final percentage of the ray that is underneath the horizon.
				float UseFulPercent = saturate((x - xr) / (xre - xr));

				UsefulMaxIteration = clamp(uint(NumSteps * UseFulPercent), 0, NumSteps);
			}
			else
			{
				UsefulMaxIteration = NumSteps;
			}
		}

		// TODO: Get the biggest from the wave to avoid increasing VGPR pressure.
		#if COMPILER_SUPPORTS_WAVE_MINMAX && 0
			MaxIteration = ToScalarMemory(WaveActiveMax(UsefulMaxIteration));
		#else
			MaxIteration = UsefulMaxIteration;
		#endif

		DebugOutput.x = float(MaxIteration) / float(NumSteps);
	}

	uint i;

	bFoundHit = false;
	bUncertain = false;

	LOOP
	for (i = 0; i < MaxIteration; i += SSRT_SAMPLE_BATCH_SIZE)
	{
		float2 SamplesUV[SSRT_SAMPLE_BATCH_SIZE];
		float4 SamplesZ;
		float4 SamplesMip;

		// Compute the sample coordinates.
		#if IS_SSGI_SHADER && SSGI_TRACE_CONE
		{
			UNROLL_N(SSRT_SAMPLE_BATCH_SIZE)
			for (uint j = 0; j < SSRT_SAMPLE_BATCH_SIZE; j++)
			{
				float S = float(i + j) + StepOffset;

				float NormalizedPower = (exp2(ExpLog2 * S) - 0.9) / MaxPower;

				float Offset = NormalizedPower * NumSteps;

				SamplesUV[j] = RayUVz.xy + Offset * RayStepUVz.xy;
				SamplesZ[j] = RayUVz.z + Offset * RayStepUVz.z;
			}
		
			SamplesMip.xy = Level;
			Level += (8.0 / NumSteps) * Roughness;
			//Level += 2.0 * ExpLog2;
		
			SamplesMip.zw = Level;
			Level += (8.0 / NumSteps) * Roughness;
			//Level += 2.0 * ExpLog2;
		}
		#else
		{
			UNROLL_N(SSRT_SAMPLE_BATCH_SIZE)
			for (uint j = 0; j < SSRT_SAMPLE_BATCH_SIZE; j++)
			{
				SamplesUV[j] = RayUVz.xy + (float(i) + float(j + 1)) * RayStepUVz.xy;
				SamplesZ[j] = RayUVz.z + (float(i) + float(j + 1)) * RayStepUVz.z;
			}
		
			SamplesMip.xy = Level;
			Level += (8.0 / NumSteps) * Roughness;
		
			SamplesMip.zw = Level;
			Level += (8.0 / NumSteps) * Roughness;
		}
		#endif

		// Sample the scene depth.
		float4 SampleDepth;
		{
			UNROLL_N(SSRT_SAMPLE_BATCH_SIZE)
			for (uint j = 0; j < SSRT_SAMPLE_BATCH_SIZE; j++)
			{
				#if DEBUG_SSRT
				{
					if (bDebugPrint)
					{
						PrintSample(InHZBUvFactorAndInvFactor, SamplesUV[j], float4(0, 1, 0, 1));
					}
				}
				#endif
				SampleDepth[j] = Texture.SampleLevel(Sampler, SamplesUV[j], SamplesMip[j]).r;
			}
		}

		// Evaluates the intersections.
		MultipleSampleDepthDiff = SamplesZ - SampleDepth;
		// Do not report a hit if we hit the far clip
		bMultipleSampleHit = and(abs(MultipleSampleDepthDiff + CompareTolerance) < CompareTolerance, SampleDepth != FarDepthValue);

		// Determines whether the ray is uncertain because might be going beind geometry.
		{
			bool4 bMultipleSampleUncertain = (MultipleSampleDepthDiff + CompareTolerance) < -CompareTolerance;

			UNROLL_N(SSRT_SAMPLE_BATCH_SIZE)
			for (uint j = 0; j < SSRT_SAMPLE_BATCH_SIZE; j++)
			{
				bFoundHit = bFoundHit || bMultipleSampleHit[j];
				bool bLocalMultisampleUncertain = bMultipleSampleUncertain[i];
				bUncertain = bUncertain || (bLocalMultisampleUncertain && !bFoundHit);
			}
		}

		BRANCH
		if (bFoundHit || (CastSettings.bStopWhenUncertain && bUncertain))
		{
			break;
		}

		LastDiff = MultipleSampleDepthDiff.w;

		//#if !SSGI_TRACE_CONE
		//	RayUVz += SSRT_SAMPLE_BATCH_SIZE * RayStepUVz;
		//#endif
	} // for( uint i = 0; i < NumSteps; i += 4 )
	
	// Compute the output coordinates.
	BRANCH
	if (bFoundHit)
    {
		#if IS_SSGI_SHADER && SSGI_TRACE_CONE
		{
			// If hit set to intersect time. If missed set to beyond end of ray
            float4 HitTime = bMultipleSampleHit ? float4(0, 1, 2, 3) : 4;

			// Take closest hit
            float Time1 = min(min3(HitTime.x, HitTime.y, HitTime.z), HitTime.w);
		
			float S = float(i + Time1) + StepOffset;

			float NormalizedPower = (exp2(log2(Exp) * S) - 0.9) / MaxPower;

			float Offset = NormalizedPower * NumSteps;

            OutHitUVz = RayUVz + RayStepUVz * Offset;
		}
		#elif IS_SSGI_SHADER
		{
			// If hit set to intersect time. If missed set to beyond end of ray
            float4 HitTime = select(bMultipleSampleHit, float4(1, 2, 3, 4), 5.0);

			// Take closest hit
            float Time1 = float(i) + min(min3(HitTime.x, HitTime.y, HitTime.z), HitTime.w);
		
            OutHitUVz = RayUVz + RayStepUVz * Time1;
		}
		#elif 0 // binary search refinement that has been attempted for SSR.
        {
			// If hit set to intersect time. If missed set to beyond end of ray
            float4 HitTime = bMultipleSampleHit ? float4(1, 2, 3, 4) : 5;

			// Take closest hit
            float Time1 = float(i) + min(min3(HitTime.x, HitTime.y, HitTime.z), HitTime.w);
            float Time0 = Time1 - 1;

            const uint NumBinarySteps = Roughness < 0.2 ? 4 : 0;

			// Binary search
            for (uint j = 0; j < NumBinarySteps; j++)
            {
                CompareTolerance *= 0.5;

                float MidTime = 0.5 * (Time0 + Time1);
                float3 MidUVz = RayUVz + RayStepUVz * MidTime;
                float MidDepth = Texture.SampleLevel(Sampler, MidUVz.xy, Level).r;
                float MidDepthDiff = MidUVz.z - MidDepth;

                if (abs(MidDepthDiff + CompareTolerance) < CompareTolerance)
                {
                    Time1 = MidTime;
                }
                else
                {
                    Time0 = MidTime;
                }
            }
			
            OutHitUVz = RayUVz + RayStepUVz * Time1;
        }
		#else // SSR
        {
            float DepthDiff0 = MultipleSampleDepthDiff[2];
            float DepthDiff1 = MultipleSampleDepthDiff[3];
            float Time0 = 3;

            FLATTEN
            if (bMultipleSampleHit[2])
            {
                DepthDiff0 = MultipleSampleDepthDiff[1];
                DepthDiff1 = MultipleSampleDepthDiff[2];
                Time0 = 2;
            }
            FLATTEN
            if (bMultipleSampleHit[1])
            {
                DepthDiff0 = MultipleSampleDepthDiff[0];
                DepthDiff1 = MultipleSampleDepthDiff[1];
                Time0 = 1;
            }
            FLATTEN
            if (bMultipleSampleHit[0])
            {
                DepthDiff0 = LastDiff;
                DepthDiff1 = MultipleSampleDepthDiff[0];
                Time0 = 0;
            }

			Time0 += float(i);

            float Time1 = Time0 + 1;
			#if 0
			{
				// Binary search
				for( uint j = 0; j < 4; j++ )
				{
					CompareTolerance *= 0.5;

					float  MidTime = 0.5 * ( Time0 + Time1 );
					float3 MidUVz = RayUVz + RayStepUVz * MidTime;
					float  MidDepth = Texture.SampleLevel( Sampler, MidUVz.xy, Level ).r;
					float  MidDepthDiff = MidUVz.z - MidDepth;

					if( abs( MidDepthDiff + CompareTolerance ) < CompareTolerance )
					{
						DepthDiff1	= MidDepthDiff;
						Time1		= MidTime;
					}
					else
					{
						DepthDiff0	= MidDepthDiff;
						Time0		= MidTime;
					}
				}
			}
			#endif

			// Find more accurate hit using line segment intersection
            float TimeLerp = saturate(DepthDiff0 / (DepthDiff0 - DepthDiff1));
            float IntersectTime = Time0 + TimeLerp;
			//float IntersectTime = lerp( Time0, Time1, TimeLerp );
				
            OutHitUVz = RayUVz + RayStepUVz * IntersectTime;
        }
		#endif
		
		#if DEBUG_SSRT
		{
			if (bDebugPrint)
				PrintSample(InHZBUvFactorAndInvFactor, OutHitUVz.xy, float4(0, 0, 1, 1));
		}
		#endif
    }
	else
    {
		// Uncertain intersection - conservatively pull back to an iteration which didn't intersect
        OutHitUVz = RayUVz + RayStepUVz * i;
    }

	OutHitUVz.xy *= InHZBUvFactorAndInvFactor.zw;
	OutHitUVz.xy = OutHitUVz.xy * float2( 2, -2 ) + float2( -1, 1 );
	OutHitUVz.xy = OutHitUVz.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;

} // CastScreenSpaceRay()


bool RayCast(
	Texture2D Texture, SamplerState Sampler,
	float3 RayOriginTranslatedWorld, float3 RayDirection,
	float Roughness, float SceneDepth,
	uint NumSteps, float StepOffset,
	float4 InHZBUvFactorAndInvFactor, 
	bool bDebugPrint,
	out float3 OutHitUVz,
	out float Level)
{
	#if IS_SSGI_SHADER
		float SlopeCompareToleranceScale = 2.0f;
	#else
		float SlopeCompareToleranceScale = 4.0f;
	#endif

	bool bRayWasClipped;
	FSSRTRay Ray = InitScreenSpaceRayFromWorldSpace(
		RayOriginTranslatedWorld, RayDirection,
		/* WorldTMax = */ SceneDepth,
		/* SceneDepth = */ SceneDepth,
		/* SlopeCompareToleranceScale = */ SlopeCompareToleranceScale,
		/* bExtendRayToScreenBorder = */ true,
		/* out */ bRayWasClipped);

	bool bHit;
	bool bUncertain;
	float3 DebugOutput;
	CastScreenSpaceRay(
		Texture, Sampler,
		/* StartMipLevel = */ 1.0,
		CreateDefaultCastSettings(),
		Ray,
		Roughness, NumSteps, StepOffset,
		InHZBUvFactorAndInvFactor, bDebugPrint,
		/* out */ DebugOutput,
		/* out */ OutHitUVz,
		/* out */ Level,
		/* out */ bHit,
		/* out */ bUncertain);

	return bHit;
} // RayCast()

float ComputeHitVignetteFromScreenPos(float2 ScreenPos)
{
	float2 Vignette = saturate(abs(ScreenPos) * 5 - 4);
	
	//PrevScreen sometimes has NaNs or Infs.  DX11 is protected because saturate turns NaNs -> 0.
	//Do a SafeSaturate so other platforms get the same protection.
	return SafeSaturate(1.0 - dot(Vignette, Vignette));
}

void ReprojectHit(float4 PrevScreenPositionScaleBias, float3 HitUVz, out float2 OutPrevUV, out float OutVignette)
{
	// Camera motion for pixel (in ScreenPos space).
	float2 ThisScreen = (HitUVz.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float4 ThisClip = float4( ThisScreen, HitUVz.z, 1 );
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float2 PrevScreen = PrevClip.xy / PrevClip.w;
	float2 PrevUV = PrevScreen.xy * PrevScreenPositionScaleBias.xy + PrevScreenPositionScaleBias.zw;

	OutVignette = min(ComputeHitVignetteFromScreenPos(ThisScreen), ComputeHitVignetteFromScreenPos(PrevScreen));
	OutPrevUV = PrevUV;
}

void ReprojectHit(float4 PrevScreenPositionScaleBias, Texture2D Texture, SamplerState Sampler, float3 HitUVz, out float2 OutPrevUV, out float OutVignette)
{
	// Camera motion for pixel (in ScreenPos space).
	float2 ThisScreen = (HitUVz.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float4 ThisClip = float4( ThisScreen, HitUVz.z, 1 );
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float2 PrevScreen = PrevClip.xy / PrevClip.w;

	float4 EncodedVelocity = Texture.SampleLevel(Sampler, HitUVz.xy, 0);
	if( EncodedVelocity.x > 0.0 )
	{
		PrevScreen = ThisClip.xy - DecodeVelocityFromTexture(EncodedVelocity).xy;
	}

	float2 PrevUV = PrevScreen.xy * PrevScreenPositionScaleBias.xy + PrevScreenPositionScaleBias.zw;
	
	OutVignette = min(ComputeHitVignetteFromScreenPos(ThisScreen), ComputeHitVignetteFromScreenPos(PrevScreen));
	OutPrevUV = PrevUV;
}

float ComputeRayHitSqrDistance(float3 OriginTranslatedWorld, float3 HitUVz)
{
	// ALU get factored out with ReprojectHit.
	float2 HitScreenPos = (HitUVz.xy - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
	float HitSceneDepth = ConvertFromDeviceZ(HitUVz.z);

	float3 HitTranslatedWorld = mul(float4(GetScreenPositionForProjectionType(HitScreenPos, HitSceneDepth), HitSceneDepth, 1), View.ScreenToTranslatedWorld).xyz;

	return length2(OriginTranslatedWorld - HitTranslatedWorld);
}

float4 SampleScreenColor(Texture2D Texture, SamplerState Sampler, float2 UV)
{
	float4 OutColor;

	OutColor.rgb = Texture.SampleLevel( Sampler, UV, 0 ).rgb;
	// Transform NaNs to black, transform negative colors to black.
	OutColor.rgb = -min(-OutColor.rgb, 0.0);
	OutColor.a = 1;
	
	return OutColor;
}

float4 SampleHCBLevel( Texture2D Texture, SamplerState Sampler, float2 UV, float Level, float4 InHZBUvFactorAndInvFactor )
{
	float4 OutColor;

	OutColor.rgb = Texture.SampleLevel( Sampler, UV * InHZBUvFactorAndInvFactor.xy, Level ).rgb;
	// Transform NaNs to black, transform negative colors to black.
	OutColor.rgb = -min(-OutColor.rgb, 0.0);
	OutColor.a = 1;

	return OutColor;
}