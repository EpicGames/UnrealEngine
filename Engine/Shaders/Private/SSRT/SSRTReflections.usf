// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSRTRayCast.ush"
#include "../Random.ush"
#include "../BRDF.ush"
#include "../MonteCarlo.ush"
#include "../SceneTextureParameters.ush"
#include "../ScreenSpaceDenoise/SSDPublic.ush"
#include "../Substrate/Substrate.ush"
#include "../HZB.ush"

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


// .r:Intensity in 0..1 range .g:RoughnessMaskMul, b:EnableDiscard for FPostProcessScreenSpaceReflectionsStencilPS, a:(bTemporalAAIsOn?TemporalAAIndex:StateFrameIndexMod8)*1551
float4 SSRParams;

float PrevSceneColorPreExposureCorrection;

// Equivalent of View.ScreenPositionScaleBias, but for SSR.
float4 PrevScreenPositionScaleBias;

Texture2D SceneColor;
SamplerState SceneColorSampler;

uint ShouldReflectOnlyWater;

float GetRoughness(const in FGBufferData GBuffer)
{
	float Roughness = GBuffer.Roughness;

	BRANCH
	if( GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT )
	{
		const float ClearCoat			= GBuffer.CustomData.x;
		const float ClearCoatRoughness	= GBuffer.CustomData.y;

		Roughness = lerp( Roughness, ClearCoatRoughness, ClearCoat );
	}

	return Roughness;
}

float GetRoughnessFade(in float Roughness)
{
	// mask SSR to reduce noise and for better performance, roughness of 0 should have SSR, at MaxRoughness we fade to 0
	return min(Roughness * SSRParams.y + 2, 1.0);
}

// Used only if r.SSR.Stencil = 1
//
// This shader generate the stencil buffer according to if a pixel needs to
// compute SSR. If it doesn't, it initializes the pixel's color.
//
// This optimization use the stencil to test to prunes out the pixel
// that dones't need to compute SSR, before pixel shader invocation are
// packed into front lines.
void ScreenSpaceReflectionsStencilPS(
	float4 SvPosition : SV_POSITION
	, out float4 OutColor : SV_Target0
#if SSR_OUTPUT_FOR_DENOISER
	, out float4 OutClosestHitDistance : SV_Target1
#endif
)
{
	float2 UV = SvPosition.xy * View.BufferSizeAndInvSize.zw;
	
#if SUBSTRATE_GBUFFER_FORMAT==1 

	uint2 PixelCoord = SvPosition.xy;
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));

	const float Roughness = TopLayerData.Roughness;
	const bool bValidMaterial = IsSubstrateMaterial(TopLayerData);

#else // SUBSTRATE_GBUFFER_FORMAT==1

	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(UV);
	const float Roughness = GetRoughness(GBuffer);
	const bool bValidMaterial = GBuffer.ShadingModelID != 0;

#endif // SUBSTRATE_GBUFFER_FORMAT==1

	const float RoughnessFade = GetRoughnessFade(Roughness);

#if !defined(TILE_COMPUTE_SSR)
	if (RoughnessFade > 0.0 && bValidMaterial)
	{
		// we are going to compute SSR for this pixel, so we discard this
		// pixel shader invocation to not overwrite the stencil buffer and
		// therefore execute ScreenSpaceReflectionsPS() for this pixel. 
		discard;
	}
#endif

	// we are not going to compute SSR for this pixel, so we clear the color
	// since ScreenSpaceReflectionsPS() won't be executed in this pixel.
	OutColor = 0;
	#if SSR_OUTPUT_FOR_DENOISER
		OutClosestHitDistance = DENOISER_INVALID_HIT_DISTANCE;
	#endif
}

void ScreenSpaceReflections(
	float4 SvPosition
	, out float4 OutColor
#if SSR_OUTPUT_FOR_DENOISER
	, out float4 OutClosestHitDistance
#endif
)
{
	float2 UV = SvPosition.xy * View.BufferSizeAndInvSize.zw;
	float2 ScreenPos = ViewportUVToScreenPos((SvPosition.xy - View.ViewRectMin.xy) * View.ViewSizeAndInvSize.zw);
	uint2 PixelPos = (uint2)SvPosition.xy;
	
    bool bDebugPrint = all(PixelPos == uint2(View.ViewSizeAndInvSize.xy) / 2);

	OutColor = 0;
	
	#if SSR_OUTPUT_FOR_DENOISER
		OutClosestHitDistance = -2.0;
	#endif

#if SUBSTRATE_GBUFFER_FORMAT==1

	uint2 PixelCoord = SvPosition.xy;

	float3 N = 0;
	float Roughness = 0.0;
	bool bValidMaterial = false;

	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));

	// For now, SSR is only used to trace the top material layer.
	N = TopLayerData.WorldNormal;
	Roughness = TopLayerData.Roughness;
	bValidMaterial = ShouldReflectOnlyWater ? IsSingleLayerWaterMaterial(TopLayerData) : IsSubstrateMaterial(TopLayerData);

	float DeviceZ = SampleDeviceZFromSceneTextures(UV);
	float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	const float3 PositionTranslatedWorld = mul(float4(GetScreenPositionForProjectionType(ScreenPos, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
	const float3 V = -GetCameraVectorFromTranslatedWorldPosition(PositionTranslatedWorld);

	// SUBSTRATE_TODO Handle anisotropy using ModifyGGXAnisotropicNormalRoughness or similar approach. 
	// Although the pass will be more expenssive if reading the material data itself, so this will need to be carefully measured on console.

#else // SUBSTRATE_GBUFFER_FORMAT==1

	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(UV);

	float3 N = GBuffer.WorldNormal;
	const float SceneDepth = GBuffer.Depth;
	const float3 PositionTranslatedWorld = mul(float4(GetScreenPositionForProjectionType(ScreenPos, SceneDepth), SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
	const float3 V = -GetCameraVectorFromTranslatedWorldPosition(PositionTranslatedWorld);

#if SUPPORTS_ANISOTROPIC_MATERIALS
	// We do not need to execute ModifyGGXAnisotropicNormalRoughness in order to handle anisotropy on clear coat material.
	// This is because SSR is only tracing the top layer which we consider to not have any anisotropy. This is when ClearCoat is 1.
	// When clear coat decreases, we smoothly fades in the anisotropy influence from the bottom layer.
	const float ClearCoat = GBuffer.CustomData.x;
	const float AnisotropyBlendValue = GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT ? ClearCoat : 0.0f;
	const float SSRAnisotropy = lerp(GBuffer.Anisotropy, 0.0f, AnisotropyBlendValue);
	ModifyGGXAnisotropicNormalRoughness(GBuffer.WorldTangent, SSRAnisotropy, GBuffer.Roughness, N, V);
#endif
	
	float Roughness = GetRoughness(GBuffer);
	const bool bValidMaterial = GBuffer.ShadingModelID != 0;

#endif // SUBSTRATE_GBUFFER_FORMAT==1

	float RoughnessFade = GetRoughnessFade(Roughness);

	// Early out. Useless if using the stencil prepass.
	BRANCH if( RoughnessFade <= 0.0 || !bValidMaterial)
	{
		return;
	}

#if SSR_QUALITY == 0
	// visualize SSR

	float PatternMask = ((PixelPos.x / 4 + PixelPos.y / 4) % 2);

	OutColor = lerp(float4(1, 0, 0, 1), float4(1, 1 ,0, 1), PatternMask) * 0.2f;
	return;
#else

	float a = Roughness * Roughness;
	float a2 = a * a;
	
	float NoV = saturate( dot( N, V ) );
	float G_SmithV = 2 * NoV / (NoV + sqrt(NoV * (NoV - NoV * a2) + a2));

	float ClosestHitDistanceSqr = INFINITE_FLOAT;

#if SSR_QUALITY == 1
	uint NumSteps = 8;
	uint NumRays = 1;
	bool bGlossy = false;
#elif SSR_QUALITY == 2
	uint NumSteps = 16;
	uint NumRays = 1;
	#if SSR_OUTPUT_FOR_DENOISER
		bool bGlossy = true;
	#else
		bool bGlossy = false;
	#endif
#elif SSR_QUALITY == 3
	uint NumSteps = 8;
	uint NumRays = 4;
	bool bGlossy = true;
#else // SSR_QUALITY == 4
	uint NumSteps = 12;
	uint NumRays = 12;
	bool bGlossy = true;
#endif
	
	if( NumRays > 1 )
	{
		float2 Noise;
		Noise.x = InterleavedGradientNoise( SvPosition.xy, View.StateFrameIndexMod8 );
		Noise.y = InterleavedGradientNoise( SvPosition.xy, View.StateFrameIndexMod8 * 117 );
	
		//uint2 Random = 0x10000 * Noise;
		uint2 Random = Rand3DPCG16( int3( PixelPos, View.StateFrameIndexMod8 ) ).xy;
			
		float3x3 TangentBasis = GetTangentBasis( N );
		float3 TangentV = mul( TangentBasis, V );

		float Count = 0;

		if( Roughness < 0.1 )
		{
			NumSteps = min( NumSteps * NumRays, 24u );
			NumRays = 1;
		}

		// Shoot multiple rays
		LOOP for( uint i = 0; i < NumRays; i++ )
		{
			float StepOffset = Noise.x;
			#if 0 // TODO
				StepOffset -= 0.9;
			#else
				StepOffset -= 0.5;
			#endif
			
			float2 E = Hammersley16( i, NumRays, Random );

#if 1
			float3 H = mul(ImportanceSampleVisibleGGX(E, a, TangentV ).xyz, TangentBasis );
			float3 L = 2 * dot( V, H ) * H - V;
#elif 0
			float3 H = mul( ImportanceSampleGGX( E, a2 ).xyz, TangentBasis );
			float3 L = 2 * dot( V, H ) * H - V;
#elif 0
			float3 L = CosineSampleHemisphere( E, N ).xyz;
#elif 0
			float3 L = CosineSampleHemisphere( E ).xyz;
			L = mul( L, TangentBasis );
#else
			float3 L;
			L.xy = UniformSampleDiskConcentric( E );
			L.z = sqrt( 1 - dot( L.xy, L.xy ) );
			L = mul( L, TangentBasis );
			float3 H = normalize(V + L);
#endif

// When 'Correct integration applies DGF' is enabled below, enable this and account for anisotropy
#if 0
			float NoL = saturate( dot(N, L) );
			float NoH = saturate( dot(N, H) );
			float VoH = saturate( dot(V, H) );
		
			float D = D_GGX( a2, NoH );
			float G_SmithL = 2 * NoL / (NoL + sqrt(NoL * (NoL - NoL * a2) + a2));
			float Vis = Vis_Smith( a2, NoV,  NoL );
			float3 F = F_Schlick( GBuffer.SpecularColor, VoH );
#endif

			float3 HitUVz;
			float Level = 0;
			
			if( Roughness < 0.1 )
			{
				L = reflect(-V, N);
			}
			
			bool bHit = RayCast(
				HZBTexture, HZBSampler,
				PositionTranslatedWorld, L, Roughness, SceneDepth, 
				NumSteps, StepOffset,
				HZBUvFactorAndInvFactor,
				bDebugPrint,
				HitUVz,
				Level
			);

#if 0	// Backface check
			if( bHit )
			{
				float3 SampleNormal = GetGBufferData( HitUVz.xy ).WorldNormal;
				bHit = dot( SampleNormal, L ) < 0;
			}
#endif

			// if there was a hit
			BRANCH if( bHit )
			{
				ClosestHitDistanceSqr = min(ClosestHitDistanceSqr, ComputeRayHitSqrDistance(PositionTranslatedWorld, HitUVz));

				float2 SampleUV;
				float Vignette;
				ReprojectHit(PrevScreenPositionScaleBias, GBufferVelocityTexture, GBufferVelocityTextureSampler, HitUVz, SampleUV, Vignette);

				float4 SampleColor = SampleScreenColor( SceneColor, SceneColorSampler, SampleUV ) * Vignette;

				SampleColor.rgb *= rcp( 1 + Luminance(SampleColor.rgb) );

				// Correct integration applies DGF below but for speed we apply EnvBRDF later when compositing
#if 0
				// CosineSampleHemisphere
				// PDF = NoL / PI,
				SampleColor.rgb *= F * (D * Vis * PI);
#elif 0
				// ImportanceSampleGGX
				// PDF = D * NoH / (4 * VoH),
				SampleColor.rgb *= F * ( NoL * Vis * (4 * VoH / NoH) );
#elif 0
				// ImportanceSampleVisibleGGX
				// PDF = G_SmithV * VoH * D / NoV / (4 * VoH)
				// PDF = G_SmithV * D / (4 * NoV);
				SampleColor.rgb *= F * G_SmithL;
#endif

				OutColor += SampleColor;
			}
		}

		//OutColor /= NumRays;
		OutColor /= max( NumRays, 0.0001 );
		OutColor.rgb *= rcp( 1 - Luminance(OutColor.rgb) );
	}
	else
	{
		float StepOffset = InterleavedGradientNoise(SvPosition.xy, View.StateFrameIndexMod8);
		#if 0 // TODO
			StepOffset -= 0.9;
		#else
			StepOffset -= 0.5;
		#endif
		
		float3 L;
		if (bGlossy)
		{
			float2 E = Rand1SPPDenoiserInput(PixelPos);
			
			#if SSR_OUTPUT_FOR_DENOISER
			{
				E.y *= 1 - GGX_IMPORTANT_SAMPLE_BIAS;
			}
			#endif
			
			float3x3 TangentBasis = GetTangentBasis( N );
			float3 TangentV = mul( TangentBasis, V );

			float3 H = mul(ImportanceSampleVisibleGGX(E, a, TangentV ).xyz, TangentBasis );
			L = 2 * dot( V, H ) * H - V;
		}
		else
		{
			L = reflect( -V, N );
		}
		
		float3 HitUVz;
		float Level = 0;
		bool bHit = RayCast(
			HZBTexture, HZBSampler,
			PositionTranslatedWorld, L, Roughness, SceneDepth,
			NumSteps, StepOffset,
			HZBUvFactorAndInvFactor,
			bDebugPrint,
			HitUVz,
			Level
		);

		BRANCH if( bHit )
		{
			ClosestHitDistanceSqr = ComputeRayHitSqrDistance(PositionTranslatedWorld, HitUVz);

			float2 SampleUV;
			float Vignette;
			ReprojectHit(PrevScreenPositionScaleBias, GBufferVelocityTexture, GBufferVelocityTextureSampler, HitUVz, SampleUV, Vignette);

			OutColor = SampleScreenColor(SceneColor, SceneColorSampler, SampleUV) * Vignette;
		}
	}
	
	OutColor *= RoughnessFade;
	OutColor *= SSRParams.r;

	OutColor.rgb *= PrevSceneColorPreExposureCorrection;
	
	// Output closest hit distance for denoiser.
	#if SSR_OUTPUT_FOR_DENOISER
	{
		OutClosestHitDistance = ComputeDenoiserConfusionFactor(
			ClosestHitDistanceSqr > 0,
			length(View.TranslatedWorldCameraOrigin - PositionTranslatedWorld),
			sqrt(ClosestHitDistanceSqr));
	}
	#endif
#endif // SSR_QUALITY == 0
}

void ScreenSpaceReflectionsPS(
	float4 SvPosition : SV_POSITION
	, out float4 OutColor : SV_Target0
#if SSR_OUTPUT_FOR_DENOISER
	, out float4 OutClosestHitDistance : SV_Target1
#endif
)
{
	ScreenSpaceReflections(SvPosition, OutColor
#if SSR_OUTPUT_FOR_DENOISER
		,OutClosestHitDistance
#endif
	);
}

#if COMPUTE_SHADER

RWTexture2D<float4> SSRColorOutput;
#if SSR_OUTPUT_FOR_DENOISER
RWTexture2D<float> SSRHitDistanceOutput;
#endif

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void ScreenSpaceReflectionsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (any(DispatchThreadId.xy >= View.ViewRectMinAndSize.zw))
	{
		return;
	}

	float4 SvPosition = float4(View.ViewRectMin.xy + DispatchThreadId.xy + float2(0.5, 0.5), 0, 1); // .zw unused
	float4 OutColor;
	float4 OutClosestHitDistance;

	ScreenSpaceReflections(SvPosition, OutColor
#if SSR_OUTPUT_FOR_DENOISER
		,OutClosestHitDistance
#endif
	);

	SSRColorOutput[View.ViewRectMinAndSize.xy + DispatchThreadId.xy] = OutColor;
#if SSR_OUTPUT_FOR_DENOISER
	SSRHitDistanceOutput[View.ViewRectMinAndSize.xy + DispatchThreadId.xy] = OutClosestHitDistance.x;
#endif
}

#endif // COMPUTE_SHADER

void VisualizeTiledScreenSpaceReflectionsPS(
	float4 SvPosition : SV_POSITION
	, out float4 OutColor : SV_Target0
#if SSR_OUTPUT_FOR_DENOISER
	, out float4 OutClosestHitDistance : SV_Target1
#endif
)
{
	ScreenSpaceReflections(SvPosition, OutColor
#if SSR_OUTPUT_FOR_DENOISER
		, OutClosestHitDistance
#endif
	);

	bool bVisualizeSSRTile = (View.FrameNumber / 30u) % 2u;
	if (bVisualizeSSRTile)
	{
		OutColor = float4(8.0f * View.PreExposure, OutColor.y, OutColor.z, 1.0f);
#if SSR_OUTPUT_FOR_DENOISER
		OutClosestHitDistance = 2.0f;
#endif
	}
}

