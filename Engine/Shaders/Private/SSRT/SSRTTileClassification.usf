// Copyright Epic Games, Inc. All Rights Reserved.

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


#define CONFIG_COLOR_TILE_CLASSIFICATION 0


#include "SSRTTileClassificationBuffer.ush"
#include "SSRTRayCast.ush"
#include "../SceneTextureParameters.ush"
#include "../FastMath.ush"
#include "../HZB.ush"


#define GROUP_TILE_SIZE 8

Texture2D ColorTexture;

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define ColorTextureSampler GlobalPointClampedSampler
#else
SamplerState ColorTextureSampler;
#endif

RWStructuredBuffer<FSSRTTileInfos> TileClassificationBufferOutput;
RWTexture2D<float4> DebugOutput;

//void StoreTileInfos(uint2 TileCoord, FSSRTTileInfos TileInfos)
//{
//	uint TileIndex = TileCoord.x + TileCoord.y * TileBufferExtent.x;
//
//	UNROLL_N(DIRECTIONALITY_COUNT)
//	for (uint i = 0; i < DIRECTIONALITY_COUNT; i++)
//	{
//		TileClassificationBufferOutput[TileIndex + ViewTileCount * i] = TileInfos.Directionality[i];
//	}
//}


[numthreads(GROUP_TILE_SIZE, GROUP_TILE_SIZE, SSRT_DIRECTIONALITY_DIVISION)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 TileCoord = DispatchThreadId;
	uint DirectionId = GroupThreadId.z;

	float2 ViewportUV = (TileCoord * GROUP_TILE_SIZE + GROUP_TILE_SIZE / 2) * View.ViewSizeAndInvSize.zw;

	float DirectionAndle = float(DirectionId) * (2 * PI * rcp(float(SSRT_DIRECTIONALITY_DIVISION)));

	float2 RayPixelDirection = float2(cos(DirectionAndle), sin(DirectionAndle));

	float2 RayStartScreen = ViewportUVToScreenPos(ViewportUV);
	float2 RayStepScreen = normalize(View.ViewSizeAndInvSize.zw * RayPixelDirection);
	RayStepScreen *= GetStepScreenFactorToClipAtScreenEdge(RayStartScreen, RayStepScreen);
	
	float2 RayEndViewportUV = ScreenPosToViewportUV(RayStartScreen + RayStepScreen);

	float2 RayStartPixel = ViewportUV * View.ViewSizeAndInvSize.xy;
	float2 RayEndPixel = RayEndViewportUV * View.ViewSizeAndInvSize.xy;
	float MaxSampleDistance = length(RayStartPixel - RayEndPixel);

	// TODO: should be furtherest depth.
	//float TileDeviceZ = FurthestHZBTexture.SampleLevel(FurthestHZBTextureSampler, RayStartPixel * SamplePixelToHZBUV, 2.0).r;
	float ClosestTileDeviceZ = ClosestHZBTexture.SampleLevel(ClosestHZBTextureSampler, RayStartPixel * SamplePixelToHZBUV, 2.0).r;
	float TileDeviceZ = ClosestTileDeviceZ;
	
	float MaxTheta = 0;

	const float DistanceToRadius = sin(2.0 * PI / (2.0 * SSRT_DIRECTIONALITY_DIVISION));
	const float d = SSRT_TILE_RES_DIVISOR;
	const float r = d * DistanceToRadius;
	const float SampleDistanceExponent = (d + r) / (d - r);

	const float MipLevelMultiplier = log2(SampleDistanceExponent);
	float CurrentSampleDistance = d;

	float PreviousMip = 0.0;

	UNROLL
	for (uint i = 0; i < 5; i++)
	{
		float TryRadius = CurrentSampleDistance * DistanceToRadius;

		// Compute the mip level directly from the radius.
		float SampleLevel = max(ceil(log2(TryRadius)), log2(d));
		float MipPow2 = pow(2.0, SampleLevel);
		
		float SamplePixelDistance = CurrentSampleDistance;
		
		CurrentSampleDistance += 2.0 * MipPow2;

		//if (i == 9) // TODO: proper stop condition.
		//{
		//	break;
		//}

		float2 SamplePixel = RayStartPixel + RayPixelDirection * min(SamplePixelDistance, SamplePixelDistance); //MaxSampleDistance);

		float SampleClosestDepths[4];
		float SampleFurthestDepths[4];
		UNROLL_N(4)
		for (uint j = 0; j < 4; j++)
		{
			const float2 Offsets = float2(j % 2, j / 2) - 0.5;
			
			float2 SceneBufferUV = (SamplePixel + Offsets * MipPow2) * View.BufferSizeAndInvSize.zw;
			if (all(SceneBufferUV == clamp(SceneBufferUV, View.BufferBilinearUVMinMax.xy, View.BufferBilinearUVMinMax.zw)))
			{
				float2 HZBUV = (SamplePixel + Offsets * MipPow2) * SamplePixelToHZBUV;
				SampleClosestDepths[j] = ClosestHZBTexture.SampleLevel(ClosestHZBTextureSampler, HZBUV, SampleLevel - 1.0).r;
				SampleFurthestDepths[j] = FurthestHZBTexture.SampleLevel(FurthestHZBTextureSampler, HZBUV, SampleLevel - 1.0).r;
			
			}
			else
			{
				SampleClosestDepths[j] = 0;
				SampleFurthestDepths[j] = 0;
			}
		}

		float SampleClosestDepth = max(
			max(SampleClosestDepths[0], SampleClosestDepths[1]),
			max(SampleClosestDepths[2], SampleClosestDepths[3]));
		float SampleFurthestDepth = min(
			min(SampleFurthestDepths[0], SampleFurthestDepths[1]),
			min(SampleFurthestDepths[2], SampleFurthestDepths[3]));
		
		//if (i == 0 || 0)
		//{
		//	float2 HZBUV = SamplePixel * SamplePixelToHZBUV;
		//	SampleClosestDepth = ClosestHZBTexture.SampleLevel(ClosestHZBTextureSampler, HZBUV, SampleLevel).r;
		//	SampleFurthestDepth = FurthestHZBTexture.SampleLevel(FurthestHZBTextureSampler, HZBUV, SampleLevel).r;
		//}

		float SampleDiviceZ = SampleClosestDepth;
		//if (i == 0 || i == 1)
		//	SampleDiviceZ = SampleFurthestDepth;
		
		// Screen space distance in viewport UV from origin of the ray.
		float DeltaU = SamplePixelDistance * View.ViewSizeAndInvSize.z;

		// Assumes inverted Z buffer.
		float DeltaZ = SampleDiviceZ - TileDeviceZ;

		float Theta = DeltaZ / DeltaU;

		MaxTheta = max(MaxTheta, Theta);
	}

	BRANCH
	if (TileCoord.x < TileBufferExtent.x && TileCoord.y < TileBufferExtent.y)
	{
		uint TileIndex = TileCoord.x + TileCoord.y * TileBufferExtent.x;
		TileClassificationBufferOutput[TileIndex].Directionality[DirectionId] = MaxTheta;

		if (DirectionId == 0)
		{
			TileClassificationBufferOutput[TileIndex].ClosestDeviceZ = ClosestTileDeviceZ;
		}
	}


	if (DirectionId == 0)
	{
		DebugOutput[TileCoord] = float4(MaxTheta / PI, TileDeviceZ, 0, 0);
	}
} // MainCS()
