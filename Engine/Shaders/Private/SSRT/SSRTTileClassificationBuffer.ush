// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSRTRayCast.ush"



uint2 TileBufferExtent;
uint ViewTileCount;
uint MaxTileCount;

StructuredBuffer<FSSRTTileInfos> TileClassificationBuffer;

/** Load tile informations using scalar load. */
FSSRTTileInfos LoadTileInfos(uint2 TileCoord)
{
	uint TileIndex = TileCoord.x + TileCoord.y * TileBufferExtent.x;

	FSSRTTileInfos TileInfos = TileClassificationBuffer[TileIndex];
	TileInfos.Coord = TileCoord;
	return TileInfos;
}


/** Test whether a screen space ray can be discarded or not based on tile information. */
bool TestRayEarlyReturn(FSSRTTileInfos TileInfos, FSSRTRay Ray)
{
	float2 RayStepPixel = Ray.RayStepScreen.xy * View.ViewSizeAndInvSize.xy;

	float DeltaU = length(RayStepPixel) * (0.5 * View.ViewSizeAndInvSize.z);
	float DeltaZ = Ray.RayStepScreen.z;
	float RayTheta = atan2(DeltaU, -DeltaZ);

	uint DirectionId =ComputeRayDirectionId(Ray);
	bool bEarlyReturn = RayTheta > TileInfos.Directionality[DirectionId];

	return bEarlyReturn;
}

