// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneCapturePixelShader.usf: 
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"

#if SUBSTRATE_ENABLED

#include "/Engine/Private/Substrate/Substrate.ush"


#define NEEDS_SCENE_TEXTURES 1
#define SCENE_TEXTURES_DISABLED 0
#define SubstratePublicStruct SubstratePublic
#include "/Engine/Public/SubstratePublic.ush"
#undef SubstratePublicStruct

#endif // SUBSTRATE_ENABLED

float4 EncodeFloatRGBA( float v )
{
	float4 Enc = float4(1.0, 255.0, 65025.0, 16581375.0) * v;
	Enc = frac(Enc);
	Enc -= Enc.yzww * 1.0/255.0;
	return Enc;
}

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
#if SOURCE_MODE_SCENE_COLOR_AND_OPACITY
	float4 SceneColorValue = CalcFullSceneColor(Input.UV);
	// Note: intentionally keeping alpha inverted, so we can scene capture to the same texture repeatedly with correct compositing
	// Also to be consistent with DrawMaterialToRenderTarget, which doesn't have this full texture copy in which to invert
	// Artists will have to invert the alpha to get foreground opacity
	OutColor = float4(SceneColorValue.rgb, saturate(SceneColorValue.a));
#elif SOURCE_MODE_SCENE_COLOR_NO_ALPHA
	OutColor = float4(CalcSceneColor(Input.UV), 0);
#elif SOURCE_MODE_SCENE_COLOR_ONE_ALPHA
	OutColor = float4(CalcSceneColor(Input.UV), 1);
#elif SOURCE_MODE_SCENE_COLOR_SCENE_DEPTH
	OutColor = float4(CalcSceneColor(Input.UV), CalcSceneDepth(Input.UV));
#elif SOURCE_MODE_SCENE_DEPTH 
	OutColor = float4(CalcSceneDepth(Input.UV), 0, 0, 0);
#elif SOURCE_MODE_DEVICE_DEPTH
	float Depth = 1 - LookupDeviceZ(Input.UV);
	OutColor.rgba = EncodeFloatRGBA(Depth).rgba;
	OutColor.a = 1;
#else

#if SUBSTRATE_GBUFFER_FORMAT==1

	uint2 ScreenPos = uint2(Input.UV * View.BufferSizeAndInvSize.xy);
	FSubstrateGBuffer SubstrateGBuffer = SubstratePublic_GetSubstrateGBufferFromFirstBSDF(ScreenPos);
#if SOURCE_MODE_NORMAL
	OutColor = float4(SubstrateGBuffer.WorldNormal, 0);
#elif SOURCE_MODE_BASE_COLOR
	OutColor = float4(SubstrateGBuffer.BaseColor, 0);
#endif

#else // SUBSTRATE_GBUFFER_FORMAT==1

	FGBufferData GBufferData = GetGBufferData(Input.UV);
#if SOURCE_MODE_NORMAL
	OutColor = float4(GBufferData.WorldNormal, 0);
#elif SOURCE_MODE_BASE_COLOR
	OutColor = float4(GBufferData.BaseColor, 0);
#endif

#endif // SUBSTRATE_GBUFFER_FORMAT==1

#endif // SOURCE_MODE_
}
