// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"

#include "/Engine/Shared/SceneCullingDefinitions.h"
#include "../BitPacking.ush"
#include "../ByteBuffer.ush"


#define SCENE_CULL_EXPLICIT_BOUNDS_BITS_PER_ELEMENT (8u) // 2x3 components a K bits each

/**
 * Unpacked cell data.
 */
struct FSceneHiearchyCellData
{
	uint3 LocalCellCoord;
	float3 LocalBoundsCenter; // Bounds center is local to the cell block
	float3 LocalBoundsExtent;
	uint BlockId;
	float MaxInstanceRadius;

	FCellBlockData BlockData;
};

// Make these compile time constants?
uint NumCellsPerBlockLog2;
uint LocalCellCoordMask;
uint CellBlockDimLog2;
int FirstLevel;
uint MaxCells;
uint NumAllocatedChunks;

StructuredBuffer<uint> InstanceIds;
StructuredBuffer<FCellBlockData> InstanceHierarchyCellBlockData;

FCellBlockData GetCellBlockData(uint BlockId)
{
	return InstanceHierarchyCellBlockData[BlockId];
}

StructuredBuffer<FPackedCellHeader> InstanceHierarchyCellHeaders;

FCellHeader UnpackCellHeader(FPackedCellHeader Packed)
{
	uint4 Bits = uint4(Packed.Packed0, Packed.Packed1, 0u, 0u);
	FCellHeader Result;
	uint Offset = 0u;
	Result.NumDynamicChunks = ReadBits(Bits, Offset, INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS); 
	Result.NumStaticChunks  = ReadBits(Bits, Offset, INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS);
	Result.ItemChunksOffset = ReadBits(Bits, Offset, INSTANCE_HIERARCHY_CELL_HEADER_OFFSET_BITS);
	
	Result.bIsValid = Result.NumDynamicChunks != 0u;
	if (Result.bIsValid)
	{
		Result.NumDynamicChunks -= 1u;
	}	
	Result.NumItemChunks = Result.NumDynamicChunks + Result.NumStaticChunks;

	return Result;
}

FCellHeader GetCellHeader(uint CellId)
{
	return UnpackCellHeader(InstanceHierarchyCellHeaders[CellId]);
}
StructuredBuffer<uint> InstanceHierarchyItemChunks;

inline uint CellIndexToBlockId(uint CellIndex)
{
	return CellIndex >> NumCellsPerBlockLog2;
}


StructuredBuffer<uint> UsedChunkIdMask;

uint IsChunkUsed(uint ChunkId)
{
	return (UsedChunkIdMask[ChunkId / 32u] & (1u << (ChunkId % 32u))) != 0u;
}

uint bCullChunkViewDistance;
ByteAddressBuffer ExplicitChunkBounds;
StructuredBuffer<uint> ExplicitChunkCellIds;

FPackedChunkAttributes LoadPackedExplicitChunkBounds(uint ChunkId)
{
	FPackedChunkAttributes Result;
	uint ReadOffset = ChunkId * SIZEOF_PACKED_CHUNK_ATTRIBUTES;
	LoadAndIncrementOffset(Result.Aabb, ExplicitChunkBounds, ReadOffset);
	LoadAndIncrementOffset(Result.InstanceDrawDistanceMinMaxSquared, ExplicitChunkBounds, ReadOffset);
	LoadAndIncrementOffset(Result.MinAnimationMinScreenSize, ExplicitChunkBounds, ReadOffset);

	return Result;
}

struct FAabb
{
	float3 Min;
	float3 Max;
};

FAabb UnpackAabb(uint2 Packed, uint BitsPerElement, float SizeScale = 1.0f)
{
	FAabb Aabb;
	Aabb.Min = UnpackToFloat3(Packed.x, BitsPerElement) * SizeScale;
	Aabb.Max = UnpackToFloat3(Packed.y, BitsPerElement) * SizeScale;
	return Aabb;
}

#define PCAF_ANY_CAST_SHADOW			(1u << 0u)
#define PCAF_ANY_SKINNED_MESH			(1u << 1u)
#define PCAF_ANY_RAYTRACING_FAR_FIELD	(1u << 2u)

struct FInstanceChunkAttributes
{
	FAabb Aabb;
	float2 InstanceDrawDistanceMinMaxSquared;
	float MinAnimationMinScreenSize;
	float MaxInstanceRadius;
	uint AnyFlags; // Flags that are true if _any_ of the instances/primitive meet the criteria
};

FInstanceChunkAttributes UnpackInstanceChunkAttributes(FPackedChunkAttributes Packed, float AabbSizeScale = 1.0f, float RadiusScale = 1.0f)
{
	FInstanceChunkAttributes Result;
	Result.Aabb = UnpackAabb(Packed.Aabb, SCENE_CULL_EXPLICIT_BOUNDS_BITS_PER_ELEMENT, AabbSizeScale);	
	Result.InstanceDrawDistanceMinMaxSquared = Packed.InstanceDrawDistanceMinMaxSquared;
	Result.MinAnimationMinScreenSize = Packed.MinAnimationMinScreenSize;
	Result.MaxInstanceRadius = BitFieldExtractFloat(Packed.Aabb.y, 8u, 24u) * RadiusScale;
	// The top 8 bits are flags
	Result.AnyFlags = Packed.Aabb.x >> 24u;
	return Result;
}


FInstanceChunkAttributes LoadInstanceChunkAttributes(uint ChunkId, float AabbSizeScale = 1.0f, float RadiusScale = 1.0f)
{
	return UnpackInstanceChunkAttributes(LoadPackedExplicitChunkBounds(ChunkId), AabbSizeScale, RadiusScale);
}


FSceneHiearchyCellData GetSceneHiearchyCellData(uint CellId)
{
	FSceneHiearchyCellData CellData;
	CellData.BlockId = CellIndexToBlockId(CellId);

	// These data sizes are small enough that we could tabulate this if it were ever important (unlikely)
	CellData.LocalCellCoord.x = CellId & LocalCellCoordMask;
	CellData.LocalCellCoord.y = (CellId >> CellBlockDimLog2) & LocalCellCoordMask;
	CellData.LocalCellCoord.z = (CellId >> (CellBlockDimLog2 * 2u)) & LocalCellCoordMask;
	// Cache the block data for future use.
	CellData.BlockData = GetCellBlockData(CellData.BlockId);
	const float LevelCellSize = CellData.BlockData.LevelCellSize;

	CellData.MaxInstanceRadius = LevelCellSize * 0.5f;
	// TODO: roll half-offset into the block world position?
	CellData.LocalBoundsCenter = float3(CellData.LocalCellCoord) * LevelCellSize + (LevelCellSize * 0.5f).xxx;
	// Note: extent is _not_ half the cell size as we have loose bounds.
	CellData.LocalBoundsExtent = LevelCellSize;
	return CellData;
}

uint UnpackChunkInstanceCount(uint PackedItemChunkDesc, inout bool bOutIsRLEPackedChunk)
{
	bOutIsRLEPackedChunk = (PackedItemChunkDesc & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
	uint NumInstances =  bOutIsRLEPackedChunk ? 64u : (PackedItemChunkDesc >> INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT);
	return NumInstances;
}

uint UnpackChunkInstanceId(bool bIsRLEPackedChunk, uint PackedItemChunkDesc, uint GroupThreadIndex)
{
	if (bIsRLEPackedChunk)
	{
		uint InstanceDataOffset = PackedItemChunkDesc & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_PAYLOAD_MASK;
		return InstanceDataOffset + GroupThreadIndex;
	}
	else
	{
		uint ChunkId = PackedItemChunkDesc & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK;
		return InstanceIds[ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE + GroupThreadIndex];
	}
}
