// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneData.ush"
#include "../WaveOpUtil.ush"
#include "../ComputeShaderUtils.ush"
#include "../ThreadGroupPrefixSum.ush"

// Move with debug shader
#include "../Visualization.ush"
#include "../ShaderPrint.ush"

#include "SceneCulling.ush"

FAabb TransformAABB(float4x4 Transform, float3 Center, float3 Extent)
{
	float3 NewCenter = mul(float4(Center, 1.0f), Transform).xyz;

	float3 NewExtent = abs(Extent.xxx * Transform[0].xyz)
		+ abs(Extent.yyy * Transform[1].xyz)
		+ abs(Extent.zzz * Transform[2].xyz);

	FAabb Result;
	Result.Min = NewCenter - NewExtent;
	Result.Max = NewCenter + NewExtent;
	return Result;
}

// pack a normalized FInstanceChunkAttributes to an uint2, using BitsPerElement for each dimension
// rounds the upper bounds up, and the lower down to ensure conservative bounds.
uint2 PackNormToUint2(FAabb NormAabb, uint BitsPerElement)
{
	checkSlow(all(NormAabb.Min >= 0.0f) && all(NormAabb.Min <= 1.0f));
	checkSlow(all(NormAabb.Max >= 0.0f) && all(NormAabb.Max <= 1.0f));

	uint2 Result = {
		PackNormToUintFloor(NormAabb.Min, BitsPerElement),
		PackNormToUintCeil(NormAabb.Max, BitsPerElement)
	};
	return Result;
}

// pack a normalized FInstanceChunkAttributes to an uint2, using BitsPerElement for each dimension
// rounds the upper bounds up, and the lower down to ensure conservative bounds.
FPackedChunkAttributes PackNormToUint2(FInstanceChunkAttributes InstanceCellAttributes)
{
	FPackedChunkAttributes Result;
	Result.Aabb = PackNormToUint2(InstanceCellAttributes.Aabb, SCENE_CULL_EXPLICIT_BOUNDS_BITS_PER_ELEMENT);
	Result.Aabb.x |= InstanceCellAttributes.AnyFlags << 24u;
	Result.Aabb.y |= PackNormToUintCeil(InstanceCellAttributes.MaxInstanceRadius, 8u) << 24u;
	// Pack?
	Result.InstanceDrawDistanceMinMaxSquared = InstanceCellAttributes.InstanceDrawDistanceMinMaxSquared;
	Result.MinAnimationMinScreenSize = InstanceCellAttributes.MinAnimationMinScreenSize;

	return Result;
}

FInstanceChunkAttributes ThreadGroupReduce(FInstanceChunkAttributes InstanceCellAttributes, uint GroupThreadIndex)
{
	FInstanceChunkAttributes Result;
	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Min.x = ThreadGroupReduceMin(InstanceCellAttributes.Aabb.Min.x, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Max.x = ThreadGroupReduceMax(InstanceCellAttributes.Aabb.Max.x, GroupThreadIndex); 

	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Min.y = ThreadGroupReduceMin(InstanceCellAttributes.Aabb.Min.y, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Max.y = ThreadGroupReduceMax(InstanceCellAttributes.Aabb.Max.y, GroupThreadIndex); 

	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Min.z = ThreadGroupReduceMin(InstanceCellAttributes.Aabb.Min.z, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.Aabb.Max.z = ThreadGroupReduceMax(InstanceCellAttributes.Aabb.Max.z, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();

	Result.InstanceDrawDistanceMinMaxSquared.x = ThreadGroupReduceMin(InstanceCellAttributes.InstanceDrawDistanceMinMaxSquared.x, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.InstanceDrawDistanceMinMaxSquared.y = ThreadGroupReduceMax(InstanceCellAttributes.InstanceDrawDistanceMinMaxSquared.y, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.AnyFlags = ThreadGroupReduceOr(InstanceCellAttributes.AnyFlags, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.MaxInstanceRadius = ThreadGroupReduceMax(InstanceCellAttributes.MaxInstanceRadius, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	Result.MinAnimationMinScreenSize = ThreadGroupReduceMin(InstanceCellAttributes.MinAnimationMinScreenSize, GroupThreadIndex); 
	GroupMemoryBarrierWithGroupSync();
	return Result;
}

void ProcessInstance(uint InstanceId, FDFVector3 WorldToBlockRelativeTranslation, inout FInstanceChunkAttributes InOutAttributes)
{
	uint PrimitiveId = 0u;
	uint InstanceFlags = 0u;
	LoadInstancePrimitiveIdAndFlags(InstanceId, PrimitiveId, InstanceFlags);

	if (PrimitiveId != INVALID_PRIMITIVE_ID)
	{
		FInstanceSceneData InstanceData = GetInstanceSceneDataUnchecked(InstanceId);
		float4x4 LocalToBlockRelativeWorld = DFFastMultiplyAffineTranslationDemote(InstanceData.LocalToWorld, WorldToBlockRelativeTranslation);
		FAabb RelBounds = TransformAABB(LocalToBlockRelativeWorld, InstanceData.LocalBoundsCenter, InstanceData.LocalBoundsExtent);
		InOutAttributes.Aabb.Min = min(InOutAttributes.Aabb.Min, RelBounds.Min);
		InOutAttributes.Aabb.Max = max(InOutAttributes.Aabb.Max, RelBounds.Max);
		InOutAttributes.MaxInstanceRadius = max(InOutAttributes.MaxInstanceRadius, length(RelBounds.Max - RelBounds.Min));

		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);
		const bool bDrawDistance = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_DRAW_DISTANCE_CULL) != 0u;
		if (bDrawDistance)
		{
			InOutAttributes.InstanceDrawDistanceMinMaxSquared.x = min(InOutAttributes.InstanceDrawDistanceMinMaxSquared.x, PrimitiveData.InstanceDrawDistanceMinMaxSquared.x);
			InOutAttributes.InstanceDrawDistanceMinMaxSquared.y = max(InOutAttributes.InstanceDrawDistanceMinMaxSquared.y, PrimitiveData.InstanceDrawDistanceMinMaxSquared.y);
		}
		else
		{
			InOutAttributes.InstanceDrawDistanceMinMaxSquared.x = 0.0f;
			InOutAttributes.InstanceDrawDistanceMinMaxSquared.y = POSITIVE_INFINITY;
		}
		if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS) != 0u)
		{
			InOutAttributes.AnyFlags |= PCAF_ANY_CAST_SHADOW;
		}	
		if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_SKINNED_MESH) != 0u)
		{
			InOutAttributes.AnyFlags |= PCAF_ANY_SKINNED_MESH;
			// Only relevant for skinned meshes
			InOutAttributes.MinAnimationMinScreenSize = min(InOutAttributes.MinAnimationMinScreenSize, PrimitiveData.AnimationMinScreenSize);
		}	
		if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_FAR_FIELD) != 0u)
		{
			InOutAttributes.AnyFlags |= PCAF_ANY_RAYTRACING_FAR_FIELD;
		}	
	}
}

#ifdef ComputeExplicitChunkBounds

int DirtyChunkCount;
StructuredBuffer<uint2> DirtyChunkBoundsData;
RWByteAddressBuffer OutExplicitChunkBoundsBuffer;
RWStructuredBuffer<uint> OutExplicitChunkCellIdsBuffer;

void StorePackedExplicitChunkBounds(uint ChunkId, FPackedChunkAttributes PackedChunkAttributes)
{
	uint WriteOffset = ChunkId * SIZEOF_PACKED_CHUNK_ATTRIBUTES;
	StoreAndIncrementOffset(PackedChunkAttributes.Aabb, OutExplicitChunkBoundsBuffer, WriteOffset);
	StoreAndIncrementOffset(PackedChunkAttributes.InstanceDrawDistanceMinMaxSquared, OutExplicitChunkBoundsBuffer, WriteOffset);
	StoreAndIncrementOffset(PackedChunkAttributes.MinAnimationMinScreenSize, OutExplicitChunkBoundsBuffer, WriteOffset);
}

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void ComputeExplicitChunkBounds(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint ChunkIdIndex = GetUnWrappedDispatchGroupId(GroupId);

	if (ChunkIdIndex >= DirtyChunkCount)
	{
		return;
	}
	uint CellId  = DirtyChunkBoundsData[ChunkIdIndex].x;
	uint ChunkId = DirtyChunkBoundsData[ChunkIdIndex].y;

	// Load cell data
	FSceneHiearchyCellData CellData = GetSceneHiearchyCellData(CellId);
	FDFVector3 WorldToBlockRelativeTranslation = DFNegate(CellData.BlockData.WorldPos);

	// Load chunk bounds (max 64)
	uint PackedItemChunkDesc = InstanceHierarchyItemChunks[ChunkId];

	bool bIsRLEPackedChunk = false;
	uint ChunkNumInstances =  UnpackChunkInstanceCount(PackedItemChunkDesc, bIsRLEPackedChunk);

	FInstanceChunkAttributes Attribs;
	Attribs.Aabb.Min = float3(POSITIVE_INFINITY, POSITIVE_INFINITY, POSITIVE_INFINITY);
	Attribs.Aabb.Max = float3(NEGATIVE_INFINITY, NEGATIVE_INFINITY, NEGATIVE_INFINITY);
	Attribs.InstanceDrawDistanceMinMaxSquared = float2(POSITIVE_INFINITY, 0.0f);
	Attribs.MinAnimationMinScreenSize = POSITIVE_INFINITY;
	Attribs.MaxInstanceRadius = 0.0f;
	Attribs.AnyFlags = 0u;

	if (GroupThreadIndex < ChunkNumInstances)
	{
		uint InstanceId = UnpackChunkInstanceId(bIsRLEPackedChunk, PackedItemChunkDesc, GroupThreadIndex);
		ProcessInstance(InstanceId, WorldToBlockRelativeTranslation, Attribs);
	}

	FInstanceChunkAttributes GroupAttribs = ThreadGroupReduce(Attribs, GroupThreadIndex);

	if (GroupThreadIndex == 0)
	{
		// store offset from min corner instead of center to get positive range [0, CellData.LocalBoundsExtent * 2]
		float3 CellMin = CellData.LocalBoundsCenter - CellData.LocalBoundsExtent;

		// min/max in positive range [0, CellData.LocalBoundsExtent * 2] -> normalized [0,1.0f)
		FInstanceChunkAttributes CellRelativeNorm;
		CellRelativeNorm.Aabb.Min = saturate((GroupAttribs.Aabb.Min - CellMin) / (CellData.LocalBoundsExtent * 2.0f));
		CellRelativeNorm.Aabb.Max = saturate((GroupAttribs.Aabb.Max - CellMin) / (CellData.LocalBoundsExtent * 2.0f));
		
		CellRelativeNorm.MaxInstanceRadius = saturate(GroupAttribs.MaxInstanceRadius / (0.5f * CellData.BlockData.LevelCellSize));

		CellRelativeNorm.InstanceDrawDistanceMinMaxSquared = GroupAttribs.InstanceDrawDistanceMinMaxSquared;
		CellRelativeNorm.MinAnimationMinScreenSize = GroupAttribs.MinAnimationMinScreenSize;
		CellRelativeNorm.AnyFlags = GroupAttribs.AnyFlags;

		StorePackedExplicitChunkBounds(ChunkId, PackNormToUint2(CellRelativeNorm));
		OutExplicitChunkCellIdsBuffer[ChunkId] = CellId;
	}
}

#endif // ComputeExplicitChunkBounds
