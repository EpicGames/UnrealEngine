// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ComputeShaderUtils.ush"
#include "../Visualization.ush"
#include "../ShaderPrint.ush"

#include "SceneCulling.ush"

StructuredBuffer<uint> ValidCellsMask;
float3 PickingRayStart;
float3 PickingRayEnd;
uint DebugMode;
RWStructuredBuffer<uint> RWDrawCellInfoCounter;

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void DebugRender(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	FShaderPrintContext Context = InitShaderPrintContext();

	const uint CellId = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, NUM_THREADS_PER_GROUP);

	if (CellId < MaxCells && (ValidCellsMask[CellId / 32] & (1u << (CellId % 32))) != 0u)
	{
		FCellHeader CellHeader = GetCellHeader(CellId);
		FSceneHiearchyCellData CellData = GetSceneHiearchyCellData(CellId);

		float3 Offset = DFDemote(CellData.BlockData.WorldPos);

		float2 RayResult = LineBoxIntersect(PickingRayStart, PickingRayEnd, CellData.LocalBoundsCenter.xyz + Offset - CellData.LocalBoundsExtent.xyz, CellData.LocalBoundsCenter.xyz + Offset + CellData.LocalBoundsExtent.xyz);
		bool bIsMouseOver = RayResult.x <= RayResult.y;

		float4 CellColor = float4(IntToColor(CellId), bIsMouseOver ? 1.0f : 0.2f);

		AddAABBWS(CellData.LocalBoundsCenter.xyz + Offset - CellData.LocalBoundsExtent.xyz, CellData.LocalBoundsCenter.xyz + Offset + CellData.LocalBoundsExtent.xyz, CellColor);

		if (bIsMouseOver)
		{
			FFontColor FontCell = InitFontColor(CellColor.xyz);
			uint StartOffsetX = 10;
			uint StartOffsetY = 30;
			uint SlotHeight = 8;
			uint SlotIndex = 0;
			InterlockedAdd(RWDrawCellInfoCounter[0], 1, SlotIndex);
			if (SlotIndex > 0 )
			{
				StartOffsetY += (SlotIndex + 2) * SlotHeight;
			}
			FShaderPrintContext PrintContext = InitShaderPrintContext(true, uint2(StartOffsetX, StartOffsetY));
			if (SlotIndex == 0)
			{
				Print(PrintContext, TEXT("FirstLevel: "), FontWhite);
				Print(PrintContext, FirstLevel);	
				Newline(PrintContext);
			}
			Print(PrintContext, TEXT("CellId: "), FontCell);
			Print(PrintContext, CellId);
			Print(PrintContext, TEXT("Level: "), FontYellow);
			Print(PrintContext, int(ceil(log2(CellData.BlockData.LevelCellSize))) - 1);
			Print(PrintContext, TEXT("NumItemChunks: "), FontYellow);
			Print(PrintContext, CellHeader.NumItemChunks);
			PrintSymbol(PrintContext, _SPC_);
			Print(PrintContext, CellHeader.NumDynamicChunks);
			Newline(PrintContext);

			{
				// don't draw chunk bounds if they are going to be identical to the cell bounds anyway (just confuses the cell color).
				if (CellHeader.NumItemChunks > 1)
				{
					uint ChunkIdStart = CellHeader.ItemChunksOffset;
					uint ChunkIdEnd = CellHeader.ItemChunksOffset + CellHeader.NumItemChunks;
					if (DebugMode == 2)
					{
						ChunkIdEnd = CellHeader.ItemChunksOffset + CellHeader.NumStaticChunks;
					}
					else if (DebugMode == 3)
					{
						ChunkIdStart += CellHeader.NumStaticChunks;
					}

					for (uint ChunkId = ChunkIdStart; ChunkId < ChunkIdEnd; ++ChunkId)
					{
						FSceneHiearchyCellData CellData = GetSceneHiearchyCellData(ExplicitChunkCellIds[ChunkId]);
						float3 Offset = DFDemote(CellData.BlockData.WorldPos);
						FInstanceChunkAttributes ChunkAttributes = LoadInstanceChunkAttributes(ChunkId, CellData.BlockData.LevelCellSize * 2.0f, CellData.BlockData.LevelCellSize * 0.5f);
						float3 ImplicitBoundsMin = float3(CellData.LocalCellCoord) * CellData.BlockData.LevelCellSize - (CellData.BlockData.LevelCellSize * 0.5f).xxx;


						float2 RayResultChunk = LineBoxIntersect(PickingRayStart, PickingRayEnd, ImplicitBoundsMin + ChunkAttributes.Aabb.Min + Offset, ImplicitBoundsMin + ChunkAttributes.Aabb.Max + Offset);
						bool bIsMouseOverChunk = RayResultChunk.x <= RayResultChunk.y;

						if (bIsMouseOverChunk)
						{
							float4 ChunkColor = float4(IntToColor(ChunkId), 1.0f);
							AddAABBWS(ImplicitBoundsMin + ChunkAttributes.Aabb.Min + Offset, ImplicitBoundsMin + ChunkAttributes.Aabb.Max + Offset, ChunkColor);

							if (ChunkAttributes.InstanceDrawDistanceMinMaxSquared.x < ChunkAttributes.InstanceDrawDistanceMinMaxSquared.y && ChunkAttributes.InstanceDrawDistanceMinMaxSquared.y < POSITIVE_INFINITY)
							{
								float3 ChunkCenter = ImplicitBoundsMin + Offset + (ChunkAttributes.Aabb.Min + ChunkAttributes.Aabb.Max) * 0.5f;
								AddSphereWS(Context, ChunkCenter, sqrt(ChunkAttributes.InstanceDrawDistanceMinMaxSquared.y), ChunkColor);
							}
						}
					}
				}
			}
		}
	}
}
