// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LargeWorldCoordinates.ush"
#include "OctahedralCommon.ush"
#include "WaveOpUtil.ush"
#include "ByteBuffer.ush"
#include "/Engine/Shared/LightSceneData.h"
#include "/Engine/Shared/SceneDefinitions.h"

#ifndef USE_GLOBAL_GPU_SCENE_DATA
	#define USE_GLOBAL_GPU_SCENE_DATA 0
#endif

#ifndef USE_GPU_SCENE_DATA_RW
	#define USE_GPU_SCENE_DATA_RW 0
#endif

#ifndef USES_PER_INSTANCE_CUSTOM_DATA
	#define USES_PER_INSTANCE_CUSTOM_DATA 0
#endif

#ifndef NEEDS_LIGHTMAP_COORDINATE
	#define NEEDS_LIGHTMAP_COORDINATE 0
#endif

#ifndef USE_INSTANCE_CULLING_DATA
	#define USE_INSTANCE_CULLING_DATA (!(RAYHITGROUPSHADER || RAYTRACING_DYNAMIC_GEOMETRY_CONVERTER))
#endif

#ifndef VF_REQUIRES_PER_INSTANCE_CUSTOM_DATA
	#define VF_REQUIRES_PER_INSTANCE_CUSTOM_DATA 0
#endif

#ifndef ENABLE_PER_INSTANCE_CUSTOM_DATA
	#define ENABLE_PER_INSTANCE_CUSTOM_DATA (USES_PER_INSTANCE_CUSTOM_DATA || VF_REQUIRES_PER_INSTANCE_CUSTOM_DATA)
#endif

#ifndef PLATFORM_USES_PRIMITIVE_UBO
	#define PLATFORM_USES_PRIMITIVE_UBO (PLATFORM_SUPPORTS_UNIFORM_BUFFER_OBJECTS && FEATURE_LEVEL == FEATURE_LEVEL_ES3_1) 
#endif

#ifndef SHADER_USES_PRIMITIVE_UBO
	#define SHADER_USES_PRIMITIVE_UBO (PLATFORM_USES_PRIMITIVE_UBO && (PIXELSHADER || VERTEXSHADER) && !USE_GLOBAL_GPU_SCENE_DATA)
#endif

// Whether to fetch primitive values (eg LocalToWorld) by dynamically indexing a scene-wide buffer, or to reference a single Primitive uniform buffer
#if VF_SUPPORTS_PRIMITIVE_SCENE_DATA
	#define VF_USE_PRIMITIVE_SCENE_DATA 1
#else
	#define VF_USE_PRIMITIVE_SCENE_DATA 0
#endif

#define USE_DITHERED_LOD_TRANSITION USE_DITHERED_LOD_TRANSITION_FROM_MATERIAL

#define INVALID_INSTANCE_PAYLOAD_OFFSET (0xFFFFFFFFu)

#if PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS
	static const uint InstanceTransformSizeFloat4Count = 2u; // compressed transform
#else
	static const uint InstanceTransformSizeFloat4Count = 3u; // encoded scale/rotation (uint4) and translation (float3)
#endif

#define NUM_CUSTOM_PRIMITIVE_DATA 9u // Num float4s used for custom data. Must match FCustomPrimitiveData::NumCustomPrimitiveDataFloat4s in SceneTypes.h

struct FPrimitiveSceneData
{
	uint		Flags;
	uint		VisibilityFlags;
	int			InstanceSceneDataOffset; // Link to the range of instances that belong to this primitive
	int			NumInstanceSceneDataEntries;
	int			PersistentPrimitiveIndex;
	uint		SingleCaptureIndex; // TODO: Use 16 bits? 8 bits?
	float3		PositionHigh;
	uint		PrimitiveComponentId; // TODO: Refactor to use PersistentPrimitiveIndex, ENGINE USE ONLY - will be removed
	FDFMatrix	LocalToWorld;
	FDFInverseMatrix WorldToLocal;
	FDFMatrix	PreviousLocalToWorld;
	FDFInverseMatrix PreviousWorldToLocal;
	float4x4	WorldToPreviousWorld; // NOTE: Intentionally not LWC
	float3		InvNonUniformScale;
	float		ObjectBoundsX;
	FDFVector3	ObjectWorldPosition;
	FDFVector3	ActorWorldPosition;
	float		MinMaterialDisplacement;
	float		MaxMaterialDisplacement;
#if PRIMITIVE_HAS_TILEOFFSET_DATA
	FLWCVector3	ObjectWorldPositionTO;
	FLWCVector3	ActorWorldPositionTO;
#endif
	float		ObjectRadius;
	uint		LightmapUVIndex;   // TODO: Use 16 bits? // TODO: Move into associated array that disappears if static lighting is disabled
	float3		ObjectOrientation; // TODO: More efficient representation?
	uint		LightmapDataIndex; // TODO: Use 16 bits? // TODO: Move into associated array that disappears if static lighting is disabled
	float4		NonUniformScale;
	float3		PreSkinnedLocalBoundsMin;
	uint		NaniteResourceID;
	float3		PreSkinnedLocalBoundsMax;
	uint		NaniteHierarchyOffset;
	uint		NaniteAssemblyTransformOffset;
	uint		NaniteAssemblyTransformCount;
	float3		LocalObjectBoundsMin;
	float		ObjectBoundsY;
	float3		LocalObjectBoundsMax;
	float		ObjectBoundsZ;
	uint		InstancePayloadDataOffset;
	uint		InstancePayloadDataStride;
	uint		InstancePayloadExtensionSize;
	float3		InstanceLocalBoundsCenter;
	float3		InstanceLocalBoundsExtent;
	float3		WireframeColor; // TODO: Should refactor out all editor data into a separate buffer
	float3		PrimitiveColor; // TODO: Should refactor out all editor data into a separate buffer
	uint		PackedNaniteFlags;
	float2 		InstanceDrawDistanceMinMaxSquared;
	float		InstanceWPODisableDistanceSquared;
	uint		NaniteRayTracingDataOffset;
	float		MaxWPOExtent;
	uint		CustomStencilValueAndMask;
	float		PixelProgrammableDistanceSquared;
	float		MaterialDisplacementFadeOutSize;
	uint2		MeshPaintTextureDescriptor;
	uint        MaterialCacheDescriptor;
	float4		CustomPrimitiveData[NUM_CUSTOM_PRIMITIVE_DATA]; // TODO: Move to associated array to shrink primitive data and pack cachelines more effectively
	float		AnimationMinScreenSize;
	float2		InstancingFadeOutParams;
};

float3 UnpackColorRGB24(float PackedColorFloat)
{
	uint PackedColor = asuint(PackedColorFloat);
	return float3
	(
		float((PackedColor >> 24u) & 0xFF),
		float((PackedColor >> 16u) & 0xFF),
		float((PackedColor >>  8u) & 0xFF)
	) * (1.0f / 255.0f);
}

// Fetch from Primitive uniform buffer
FPrimitiveSceneData GetPrimitiveDataFromUniformBuffer()
{
	FPrimitiveSceneData PrimitiveData;

	PrimitiveData.Flags								= Primitive.Flags;
	PrimitiveData.VisibilityFlags					= Primitive.VisibilityFlags;
	PrimitiveData.InstanceSceneDataOffset			= Primitive.InstanceSceneDataOffset;
	PrimitiveData.NumInstanceSceneDataEntries		= Primitive.NumInstanceSceneDataEntries;
	PrimitiveData.SingleCaptureIndex				= Primitive.SingleCaptureIndex;
	PrimitiveData.PositionHigh						= Primitive.PositionHigh;
	PrimitiveData.PrimitiveComponentId				= Primitive.PrimitiveComponentId;
	
	PrimitiveData.LocalToWorld						= MakeDFMatrix4x3(Primitive.PositionHigh, Primitive.LocalToRelativeWorld);
	PrimitiveData.WorldToLocal						= MakeDFInverseMatrix4x3(Primitive.PositionHigh, Primitive.RelativeWorldToLocal);
	PrimitiveData.PreviousLocalToWorld				= MakeDFMatrix4x3(Primitive.PositionHigh, Primitive.PreviousLocalToRelativeWorld);
	PrimitiveData.PreviousWorldToLocal				= MakeDFInverseMatrix4x3(Primitive.PositionHigh, Primitive.PreviousRelativeWorldToLocal);

	PrimitiveData.WorldToPreviousWorld				= Primitive.WorldToPreviousWorld;
	PrimitiveData.InvNonUniformScale				= Primitive.InvNonUniformScale;
	PrimitiveData.ObjectBoundsX						= Primitive.ObjectBoundsX;
	PrimitiveData.ObjectRadius						= Primitive.ObjectWorldPositionHighAndRadius.w;
#if PRIMITIVE_HAS_TILEOFFSET_DATA
	PrimitiveData.ObjectWorldPositionTO				= MakeLWCVector3(Primitive.ObjectWorldPositionHighAndRadius.xyz, Primitive.ObjectWorldPositionLow.xyz);
	PrimitiveData.ActorWorldPositionTO				= MakeLWCVector3(Primitive.ActorWorldPositionHigh.xyz, Primitive.ActorWorldPositionLow.xyz);
	PrimitiveData.ObjectWorldPosition				= DFFromTileOffset(PrimitiveData.ObjectWorldPositionTO);
	PrimitiveData.ActorWorldPosition				= DFFromTileOffset(PrimitiveData.ActorWorldPositionTO);
#else
	PrimitiveData.ObjectWorldPosition				= MakeDFVector3(Primitive.ObjectWorldPositionHighAndRadius.xyz, Primitive.ObjectWorldPositionLow.xyz);
	PrimitiveData.ActorWorldPosition				= MakeDFVector3(Primitive.ActorWorldPositionHigh.xyz, Primitive.ActorWorldPositionLow.xyz);
#endif
	PrimitiveData.LightmapUVIndex					= Primitive.LightmapUVIndex;
	PrimitiveData.ObjectOrientation					= Primitive.ObjectOrientation;
	PrimitiveData.LightmapDataIndex					= Primitive.LightmapDataIndex;
	PrimitiveData.NonUniformScale					= Primitive.NonUniformScale;
	PrimitiveData.PreSkinnedLocalBoundsMin			= Primitive.PreSkinnedLocalBoundsMin;
	PrimitiveData.NaniteResourceID					= Primitive.NaniteResourceID;
	PrimitiveData.PreSkinnedLocalBoundsMax			= Primitive.PreSkinnedLocalBoundsMax;
	PrimitiveData.NaniteHierarchyOffset				= Primitive.NaniteHierarchyOffset;
	PrimitiveData.NaniteAssemblyTransformOffset		= Primitive.NaniteAssemblyTransformOffset;
	PrimitiveData.NaniteAssemblyTransformCount		= Primitive.NaniteAssemblyTransformCount;
	PrimitiveData.LocalObjectBoundsMin				= Primitive.LocalObjectBoundsMin;
	PrimitiveData.ObjectBoundsY						= Primitive.ObjectBoundsY;
	PrimitiveData.LocalObjectBoundsMax				= Primitive.LocalObjectBoundsMax;
	PrimitiveData.ObjectBoundsZ						= Primitive.ObjectBoundsZ;
	PrimitiveData.InstancePayloadDataOffset			= Primitive.InstancePayloadDataOffset;
	PrimitiveData.InstancePayloadDataStride			= Primitive.InstancePayloadDataStride;
	PrimitiveData.InstancePayloadExtensionSize		= Primitive.InstancePayloadExtensionSize;
	PrimitiveData.InstanceLocalBoundsCenter			= Primitive.InstanceLocalBoundsCenter;
	PrimitiveData.InstanceLocalBoundsExtent			= Primitive.InstanceLocalBoundsExtent;
	PrimitiveData.WireframeColor					= UnpackColorRGB24(Primitive.WireframeAndPrimitiveColor.x);
	PrimitiveData.PrimitiveColor					= UnpackColorRGB24(Primitive.WireframeAndPrimitiveColor.y);
	PrimitiveData.PackedNaniteFlags					= Primitive.PackedNaniteFlags;
	PrimitiveData.InstanceDrawDistanceMinMaxSquared	= Primitive.InstanceDrawDistanceMinMaxSquared;
	PrimitiveData.InstanceWPODisableDistanceSquared	= Primitive.InstanceWPODisableDistanceSquared;
	PrimitiveData.PersistentPrimitiveIndex			= Primitive.PersistentPrimitiveIndex;
	PrimitiveData.NaniteRayTracingDataOffset		= Primitive.NaniteRayTracingDataOffset;
	PrimitiveData.MaxWPOExtent						= Primitive.MaxWPOExtent;
	PrimitiveData.MinMaterialDisplacement			= Primitive.MinMaterialDisplacement;
	PrimitiveData.MaxMaterialDisplacement			= Primitive.MaxMaterialDisplacement;
	PrimitiveData.CustomStencilValueAndMask			= Primitive.CustomStencilValueAndMask;
	PrimitiveData.PixelProgrammableDistanceSquared	= Primitive.PixelProgrammableDistanceSquared;
	PrimitiveData.MaterialDisplacementFadeOutSize	= Primitive.MaterialDisplacementFadeOutSize;
	PrimitiveData.MeshPaintTextureDescriptor		= Primitive.MeshPaintTextureDescriptor;
	PrimitiveData.MaterialCacheDescriptor			= Primitive.MaterialCacheDescriptor;
	PrimitiveData.AnimationMinScreenSize			= Primitive.AnimationMinScreenSize;
	PrimitiveData.InstancingFadeOutParams			= Primitive.InstancingFadeOutParams;
	
	UNROLL
	for (uint DataIndex = 0; DataIndex < NUM_CUSTOM_PRIMITIVE_DATA; ++DataIndex)
	{
		PrimitiveData.CustomPrimitiveData[DataIndex] = Primitive.CustomPrimitiveData[DataIndex];
	}

	return PrimitiveData;
}

// Unpacked AoS layout - see FInstanceSceneShaderData::Setup() for SoA packed layout.
#if USE_EDITOR_SHADERS
struct FInstanceSceneEditorData
{
	float3 HitProxyId;
	uint   HitProxyPacked;
	bool bIsSelected;
};
#endif

struct FInstanceSceneData
{
	uint InstanceId; // Note: derived data (not loaded from scene, but set to the ID used to load the data for convenience).
	FDFMatrix LocalToWorld;
	FDFMatrix PrevLocalToWorld;
	FDFInverseMatrix WorldToLocal;
	float4   NonUniformScale;
	float3   InvNonUniformScale;
	float    DeterminantSign;
	float3   LocalBoundsCenter;
	uint     PrimitiveId;
	uint     RelativeId;
	uint     PayloadDataOffset;
	uint     PayloadExtensionOffset;
	uint     PayloadExtensionSize;
	float3   LocalBoundsExtent;
	uint     LastUpdateSceneFrameNumber;
	uint     NaniteRuntimeResourceID;
	uint     NaniteHierarchyOffset;
	uint     SkinningData;
	float    RandomID;
#if ENABLE_PER_INSTANCE_CUSTOM_DATA
	uint     CustomDataOffset;
	uint     CustomDataCount;
#endif
#if 1 //NEEDS_LIGHTMAP_COORDINATE // TODO: Fix Me
	float4   LightMapAndShadowMapUVBias;
#endif
	bool     ValidInstance;
	uint     Flags;

#if USE_EDITOR_SHADERS
	FInstanceSceneEditorData EditorData;
#endif
};

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
FInstanceSceneData WaveReadLaneAt(FInstanceSceneData In, uint SrcIndex)
{
	FInstanceSceneData Result;

	Result.InstanceId						= WaveReadLaneAt(In.InstanceId, SrcIndex);
	Result.LocalToWorld						= WaveReadLaneAt(In.LocalToWorld, SrcIndex);
	Result.PrevLocalToWorld					= WaveReadLaneAt(In.PrevLocalToWorld, SrcIndex);
	Result.WorldToLocal						= WaveReadLaneAt(In.WorldToLocal, SrcIndex);
	Result.NonUniformScale					= WaveReadLaneAt(In.NonUniformScale, SrcIndex);
	Result.InvNonUniformScale				= WaveReadLaneAt(In.InvNonUniformScale, SrcIndex);
	Result.DeterminantSign					= WaveReadLaneAt(In.DeterminantSign, SrcIndex);
	Result.LocalBoundsCenter				= WaveReadLaneAt(In.LocalBoundsCenter, SrcIndex);
	Result.PrimitiveId						= WaveReadLaneAt(In.PrimitiveId, SrcIndex);
	Result.RelativeId						= WaveReadLaneAt(In.RelativeId, SrcIndex);
	Result.PayloadDataOffset				= WaveReadLaneAt(In.PayloadDataOffset, SrcIndex);
	Result.PayloadExtensionOffset			= WaveReadLaneAt(In.PayloadExtensionOffset, SrcIndex);
	Result.PayloadExtensionSize				= WaveReadLaneAt(In.PayloadExtensionSize, SrcIndex);
	Result.LocalBoundsExtent				= WaveReadLaneAt(In.LocalBoundsExtent, SrcIndex);
	Result.LocalBoundsCenter				= WaveReadLaneAt(In.LocalBoundsCenter, SrcIndex);
	Result.PrimitiveId						= WaveReadLaneAt(In.PrimitiveId, SrcIndex);
	Result.RelativeId						= WaveReadLaneAt(In.RelativeId, SrcIndex);
	Result.PayloadDataOffset				= WaveReadLaneAt(In.PayloadDataOffset, SrcIndex);
	Result.PayloadExtensionOffset			= WaveReadLaneAt(In.PayloadExtensionOffset, SrcIndex);
	Result.PayloadExtensionSize				= WaveReadLaneAt(In.PayloadExtensionSize, SrcIndex);
	Result.LocalBoundsExtent				= WaveReadLaneAt(In.LocalBoundsExtent, SrcIndex);
	Result.LastUpdateSceneFrameNumber		= WaveReadLaneAt(In.LastUpdateSceneFrameNumber, SrcIndex);
	Result.NaniteRuntimeResourceID			= WaveReadLaneAt(In.NaniteRuntimeResourceID, SrcIndex);
	Result.NaniteHierarchyOffset			= WaveReadLaneAt(In.NaniteHierarchyOffset, SrcIndex);
	Result.SkinningData						= WaveReadLaneAt(In.SkinningData, SrcIndex);
	Result.RandomID							= WaveReadLaneAt(In.RandomID, SrcIndex);
#if ENABLE_PER_INSTANCE_CUSTOM_DATA
	Result.CustomDataOffset					= WaveReadLaneAt(In.CustomDataOffset, SrcIndex);
	Result.CustomDataCount					= WaveReadLaneAt(In.CustomDataCount, SrcIndex);
#endif
	Result.LightMapAndShadowMapUVBias		= WaveReadLaneAt(In.LightMapAndShadowMapUVBias, SrcIndex);
	Result.ValidInstance					= WaveReadLaneAt(In.ValidInstance, SrcIndex);
	Result.Flags							= WaveReadLaneAt(In.Flags, SrcIndex);
#if USE_EDITOR_SHADERS
	//TODO
#endif

	return Result;
}
#endif

#if VF_USE_PRIMITIVE_SCENE_DATA

#if SHADER_USES_PRIMITIVE_UBO
// no access to global buffers
#elif USE_GLOBAL_GPU_SCENE_DATA
StructuredBuffer<float4> GPUScenePrimitiveSceneData;
#elif USE_GPU_SCENE_DATA_RW
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
RWStructuredBuffer<float4> GPUScenePrimitiveSceneDataRW;
#endif

#endif

float4 LoadPrimitivePrimitiveSceneDataElement(uint PrimitiveIndex, uint ItemIndex)
{
	uint TargetIdx = PrimitiveIndex + ItemIndex;

#if SHADER_USES_PRIMITIVE_UBO
	return 0; // FIXME LoadPrimitivePrimitiveSceneDataElement for UBO case
#elif USE_GLOBAL_GPU_SCENE_DATA
	checkStructuredBufferAccessSlow(GPUScenePrimitiveSceneData, TargetIdx);
	return GPUScenePrimitiveSceneData[TargetIdx];
#elif USE_GPU_SCENE_DATA_RW
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	checkStructuredBufferAccessSlow(GPUScenePrimitiveSceneDataRW, TargetIdx);
	return GPUScenePrimitiveSceneDataRW[TargetIdx];
#else
	checkStructuredBufferAccessSlow(GPUSceneWriter.GPUScenePrimitiveSceneDataRW, TargetIdx);
	return GPUSceneWriter.GPUScenePrimitiveSceneDataRW[TargetIdx];
#endif

#else
	checkStructuredBufferAccessSlow(Scene.GPUScene.GPUScenePrimitiveSceneData, TargetIdx);
	return Scene.GPUScene.GPUScenePrimitiveSceneData[TargetIdx];
#endif
}

#if SHADER_USES_PRIMITIVE_UBO
#include "SceneDataMobileLoader.ush"
#endif

// Fetch from scene primitive buffer
FPrimitiveSceneData GetPrimitiveData(uint PrimitiveId)
{
#if SHADER_USES_PRIMITIVE_UBO
	#if USE_INSTANCE_CULLING
		return GetPrimitiveDataFromUniformBuffer();
	#else
		return LoadPrimitiveDataUBO(PrimitiveId);
	#endif
#elif (VERTEXSHADER && !PLATFORM_SUPPORTS_VERTEX_SHADER_SRVS)
	// Vertex shaders may not have access to GPUScene on mobile. Use GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
	// TODO: need a way to report invalid usage, after all dead code elimination
	return (FPrimitiveSceneData)0;
#else
	FPrimitiveSceneData PrimitiveData = (FPrimitiveSceneData)0;

	// short-circuit to avoid redundant indirection (as this is always the same value it is no longer stored in the buffer)
	PrimitiveData.PersistentPrimitiveIndex = PrimitiveId;

	// Note: layout must match FPrimitiveSceneShaderData::Setup in C++
	// Relying on optimizer to remove unused loads
	uint PrimitiveIndex = PrimitiveId * PRIMITIVE_SCENE_DATA_STRIDE;
	
	float3 PositionHigh = LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 1).xyz;

	const float4x4 LocalToWorld = transpose(float4x4(
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 2),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 3),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 4),
		float4(0, 0, 0, 1)
	));

	const float4x4 PreviousLocalToWorld = transpose(float4x4(
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 8),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 9),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 10),
		float4(0, 0, 0, 1)
	));

	const float4x4 WorldToLocal = transpose(float4x4(
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 5),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 6),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 7),
		float4(0, 0, 0, 1)
	));

	const float4x4 PreviousWorldToLocal = transpose(float4x4(
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 11),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 12),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 13),
		float4(0, 0, 0, 1)
	));

	const float4x4 WorldToPreviousWorld = transpose(float4x4(
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 14),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 15),
		LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 16),
		float4(0, 0, 0, 1)
	));

	float4 ObjectWorldPositionHighAndRadius	= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 18);
	float4 ObjectWorldPositionLow			= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 19);

	float4 ActorWorldPositionHigh	= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 20);
	float4 ActorWorldPositionLow	= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 21);

	PrimitiveData.Flags									= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 0).x);
	PrimitiveData.InstanceSceneDataOffset				= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 0).y);
	PrimitiveData.NumInstanceSceneDataEntries			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 0).z);
	PrimitiveData.SingleCaptureIndex					= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 0).w) & 0xFFFFu;
	PrimitiveData.VisibilityFlags						= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 0).w) >> 16u;
	PrimitiveData.PositionHigh							= PositionHigh; // 1.xyz
	PrimitiveData.PrimitiveComponentId					= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 1).w);
	PrimitiveData.LocalToWorld							= MakeDFMatrix4x3(PositionHigh, LocalToWorld);
	PrimitiveData.WorldToLocal							= MakeDFInverseMatrix4x3(PositionHigh, WorldToLocal);
	PrimitiveData.PreviousLocalToWorld					= MakeDFMatrix4x3(PositionHigh, PreviousLocalToWorld);
	PrimitiveData.PreviousWorldToLocal					= MakeDFInverseMatrix4x3(PositionHigh, PreviousWorldToLocal);
	PrimitiveData.WorldToPreviousWorld					= WorldToPreviousWorld;
	PrimitiveData.InvNonUniformScale					= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 17).xyz;
	PrimitiveData.ObjectBoundsX							= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 17).w;
	PrimitiveData.ObjectRadius							= ObjectWorldPositionHighAndRadius.w;
#if PRIMITIVE_HAS_TILEOFFSET_DATA
	PrimitiveData.ObjectWorldPositionTO					= MakeLWCVector3(ObjectWorldPositionHighAndRadius.xyz, ObjectWorldPositionLow.xyz);
	PrimitiveData.ActorWorldPositionTO					= MakeLWCVector3(ActorWorldPositionHigh.xyz, ActorWorldPositionLow.xyz);
	PrimitiveData.ObjectWorldPosition					= DFFromTileOffset(PrimitiveData.ObjectWorldPositionTO);
	PrimitiveData.ActorWorldPosition					= DFFromTileOffset(PrimitiveData.ActorWorldPositionTO);
#else
	PrimitiveData.ObjectWorldPosition					= MakeDFVector3(ObjectWorldPositionHighAndRadius.xyz, ObjectWorldPositionLow.xyz);
	PrimitiveData.ActorWorldPosition					= MakeDFVector3(ActorWorldPositionHigh.xyz, ActorWorldPositionLow.xyz);
#endif
	PrimitiveData.MinMaterialDisplacement				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 19).w;
	PrimitiveData.MaxMaterialDisplacement				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 20).w;
	PrimitiveData.LightmapUVIndex						= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 21).w);
	PrimitiveData.ObjectOrientation						= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 22).xyz;
	PrimitiveData.LightmapDataIndex						= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 22).w);
	PrimitiveData.NonUniformScale						= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 23);
	PrimitiveData.PreSkinnedLocalBoundsMin				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 24).xyz;
	PrimitiveData.NaniteResourceID						= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 24).w);
	PrimitiveData.PreSkinnedLocalBoundsMax				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 25).xyz;
	PrimitiveData.NaniteHierarchyOffset					= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 25).w);
	PrimitiveData.LocalObjectBoundsMin					= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 26).xyz;
	PrimitiveData.ObjectBoundsY							= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 26).w;
	PrimitiveData.LocalObjectBoundsMax					= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 27).xyz;
	PrimitiveData.ObjectBoundsZ							= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 27).w;
	PrimitiveData.InstanceLocalBoundsCenter				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 28).xyz;
	PrimitiveData.InstancePayloadDataOffset				= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 28).w);
	PrimitiveData.InstanceLocalBoundsExtent				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 29).xyz;
	PrimitiveData.InstancePayloadDataStride				= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 29).w) & 0x00FFFFFFu;
	PrimitiveData.InstancePayloadExtensionSize			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 29).w) >> 24;
	PrimitiveData.WireframeColor						= UnpackColorRGB24(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 30).x);
	PrimitiveData.PrimitiveColor						= UnpackColorRGB24(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 30).y);
	PrimitiveData.PackedNaniteFlags						= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 30).z);
	// 30.w is unused
	PrimitiveData.InstanceDrawDistanceMinMaxSquared		= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 31).xy;
	PrimitiveData.InstanceWPODisableDistanceSquared		= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 31).z;
	PrimitiveData.NaniteRayTracingDataOffset			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 31).w);
	PrimitiveData.MaxWPOExtent							= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 32).x;
	PrimitiveData.CustomStencilValueAndMask				= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 32).y);
	PrimitiveData.PixelProgrammableDistanceSquared		= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 32).z;
	PrimitiveData.MaterialDisplacementFadeOutSize		= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 32).w;
	PrimitiveData.MeshPaintTextureDescriptor			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 33).xy);
	PrimitiveData.NaniteAssemblyTransformOffset			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 33).z);
	PrimitiveData.NaniteAssemblyTransformCount			= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 33).w);
	PrimitiveData.MaterialCacheDescriptor				= asuint(LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 34).x);
	PrimitiveData.AnimationMinScreenSize				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 34).y;
	PrimitiveData.InstancingFadeOutParams				= LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex, 34).zw;
	// NOTE: Please make sure GetPrimitiveDataFromUniformBuffer() gets updated as well when adding new members here

	// TODO: Move to associated array (and editor data) to shrink primitive data and better pack cachelines
	UNROLL
	for (uint DataIndex = 0; DataIndex < NUM_CUSTOM_PRIMITIVE_DATA; ++DataIndex)
	{
		PrimitiveData.CustomPrimitiveData[DataIndex] = LoadPrimitivePrimitiveSceneDataElement(PrimitiveIndex,  35 + DataIndex);
	}
	return PrimitiveData;
#endif
}

#else // !VF_USE_PRIMITIVE_SCENE_DATA

FPrimitiveSceneData GetPrimitiveData(uint PrimitiveId)
{
	return GetPrimitiveDataFromUniformBuffer();
}

#endif // VF_USE_PRIMITIVE_SCENE_DATA

float GetPrimitive_DeterminantSign_FromFlags(uint Flags)
{
	return CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_DETERMINANT_SIGN, -1.0f, 1.0f);
}

float GetPrimitive_DeterminantSign(uint PrimitiveId)
{
	return GetPrimitive_DeterminantSign_FromFlags(GetPrimitiveData(PrimitiveId).Flags);
}

#if VF_USE_PRIMITIVE_SCENE_DATA
float GetPrimitive_DeterminantSign(FPrimitiveSceneData Primitive)
{
	return GetPrimitive_DeterminantSign_FromFlags(Primitive.Flags);
}
#endif

float GetPrimitive_PerObjectGBufferData_FromFlags(uint Flags)
{
	const float CapsuleRepresentation = CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION, 1.0f, 0.0f);
	const float CastContactShadow     = CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW,    1.0f, 0.0f);
	return (2.0f * CapsuleRepresentation + CastContactShadow) / 3.0f;
}

float GetPrimitive_PerObjectGBufferData(uint PrimitiveId)
{
	return GetPrimitive_PerObjectGBufferData_FromFlags(GetPrimitiveData(PrimitiveId).Flags);
}

#if VF_USE_PRIMITIVE_SCENE_DATA
float GetPrimitive_PerObjectGBufferData(FPrimitiveSceneData Primitive)
{
	return GetPrimitive_PerObjectGBufferData_FromFlags(Primitive.Flags);
}
#endif

uint GetPrimitive_LightingChannelMask_FromFlags(uint Flags)
{
	const uint Channel0 = CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_0, 1u, 0u);
	const uint Channel1 = CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_1, 1u, 0u);
	const uint Channel2 = CondMask(Flags & PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_2, 1u, 0u);
	return (Channel0 | (Channel1 << 1u) | (Channel2 << 2u));
}

uint GetPrimitive_LightingChannelMask(uint PrimitiveId)
{
	return GetPrimitive_LightingChannelMask_FromFlags(GetPrimitiveData(PrimitiveId).Flags);
}

#if VF_USE_PRIMITIVE_SCENE_DATA
uint GetPrimitive_LightingChannelMask(FPrimitiveSceneData Primitive)
{
	return GetPrimitive_LightingChannelMask_FromFlags(Primitive.Flags);
}
#endif

float3 UnpackHitProxyId(uint HitProxyPacked)
{
	// BGR (dword) -> RGA (float)
	return float3
	(
		float((HitProxyPacked       ) & 0xFF),
		float((HitProxyPacked >>  8u) & 0xFF),
		float((HitProxyPacked >> 16u) & 0xFF)
	) * (1.0f / 255.0f);
}

struct FInstancePayloadDataOffsets
{
	uint HierarchyOffset;
	uint EditorData;
	uint LocalBounds;
	uint DynamicData;
	uint SkinningData;
	uint LightShadowUVBias;
	uint PayloadExtension;
	uint CustomData;
};

#if SHADER_USES_PRIMITIVE_UBO
// no access to global buffers
#elif USE_GLOBAL_GPU_SCENE_DATA
StructuredBuffer<float4> GPUSceneInstanceSceneData;
StructuredBuffer<float4> GPUSceneInstancePayloadData;
uint GPUSceneFrameNumber;
uint GPUSceneMaxPersistentPrimitiveIndex;
uint GPUSceneInstanceDataTileSizeLog2;
uint GPUSceneInstanceDataTileSizeMask;
uint GPUSceneInstanceDataTileStride;

uint GPUSceneMaxAllocatedInstanceId;
uint GPUSceneNumLightmapDataItems;
#elif USE_GPU_SCENE_DATA_RW
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
RWStructuredBuffer<float4> GPUSceneInstanceSceneDataRW;
RWStructuredBuffer<float4> GPUSceneInstancePayloadDataRW;
uint GPUSceneFrameNumber;
uint GPUSceneInstanceDataTileSizeLog2;
uint GPUSceneInstanceDataTileSizeMask;
uint GPUSceneInstanceDataTileStride;
uint GPUSceneMaxPersistentPrimitiveIndex;
uint GPUSceneMaxAllocatedInstanceId;
uint GPUSceneNumLightmapDataItems;
#endif
#endif

struct FSceneData
{
	uint FrameNumber;
	/** Upper bound that includes all the IDs of allocated primitives in the Scene (NOT dynamic). This is always >= the number of primitives in the scene as there may be holes. */
	uint MaxPersistentPrimitiveIndex;
	/** Upper bound that includes all the IDs of ALL allocated instances (scene & dynamic). This is always >= the number of instances in the scene as there may be holes. */
	uint MaxAllocatedInstanceId;

	uint NumLightmapDataItems;

	// implementation details, use at own risk
	uint InstanceDataTileSizeLog2;
	uint InstanceDataTileSizeMask;
	uint InstanceDataTileStride;
};

FSceneData GetSceneData()
{
	FSceneData SceneData;
#if USE_GLOBAL_GPU_SCENE_DATA
	SceneData.FrameNumber = GPUSceneFrameNumber;
	SceneData.InstanceDataTileSizeLog2 = GPUSceneInstanceDataTileSizeLog2;
	SceneData.InstanceDataTileSizeMask = GPUSceneInstanceDataTileSizeMask;
	SceneData.InstanceDataTileStride = GPUSceneInstanceDataTileStride;
	SceneData.MaxPersistentPrimitiveIndex = GPUSceneMaxPersistentPrimitiveIndex;
	SceneData.MaxAllocatedInstanceId = GPUSceneMaxAllocatedInstanceId;
	SceneData.NumLightmapDataItems = GPUSceneNumLightmapDataItems;
#elif USE_GPU_SCENE_DATA_RW

#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	SceneData.FrameNumber = GPUSceneFrameNumber;
	SceneData.InstanceDataTileSizeLog2 = GPUSceneInstanceDataTileSizeLog2;
	SceneData.InstanceDataTileSizeMask = GPUSceneInstanceDataTileSizeMask;
	SceneData.InstanceDataTileStride = GPUSceneInstanceDataTileStride;
	SceneData.MaxPersistentPrimitiveIndex = GPUSceneMaxPersistentPrimitiveIndex;
	SceneData.MaxAllocatedInstanceId = GPUSceneMaxAllocatedInstanceId;
	SceneData.NumLightmapDataItems = GPUSceneNumLightmapDataItems;
#else
	SceneData.FrameNumber = GPUSceneWriter.GPUSceneFrameNumber;
	SceneData.InstanceDataTileSizeLog2 = GPUSceneWriter.InstanceDataTileSizeLog2;
	SceneData.InstanceDataTileSizeMask = GPUSceneWriter.InstanceDataTileSizeMask;
	SceneData.InstanceDataTileStride = GPUSceneWriter.InstanceDataTileStride;
	SceneData.MaxPersistentPrimitiveIndex = GPUSceneWriter.GPUSceneMaxPersistentPrimitiveIndex;
	SceneData.MaxAllocatedInstanceId = GPUSceneWriter.GPUSceneMaxAllocatedInstanceId;
	SceneData.NumLightmapDataItems = GPUSceneWriter.GPUSceneNumLightmapDataItems;
#endif

#else
	SceneData.FrameNumber = Scene.GPUScene.GPUSceneFrameNumber;
	SceneData.InstanceDataTileSizeLog2 = Scene.GPUScene.GPUSceneInstanceDataTileSizeLog2;
	SceneData.InstanceDataTileSizeMask = Scene.GPUScene.GPUSceneInstanceDataTileSizeMask;
	SceneData.InstanceDataTileStride = Scene.GPUScene.GPUSceneInstanceDataTileStride;
	SceneData.MaxPersistentPrimitiveIndex = Scene.GPUScene.GPUSceneMaxPersistentPrimitiveIndex;
	SceneData.MaxAllocatedInstanceId = Scene.GPUScene.GPUSceneMaxAllocatedInstanceId;
	SceneData.NumLightmapDataItems = Scene.GPUScene.GPUSceneNumLightmapDataItems;
#endif
	return SceneData;
}

uint GetGPUSceneFrameNumber()
{
	return GetSceneData().FrameNumber;
}

/**
 * Function to calculate the instance data index in the instance data buffer given the instance ID and array index
 */
uint CalcInstanceDataIndex(uint InstanceId, uint ArrayIndex)
{
	FSceneData SceneData = GetSceneData();
	uint TileId = InstanceId >> SceneData.InstanceDataTileSizeLog2;
	uint IdInTile = InstanceId & SceneData.InstanceDataTileSizeMask;
	return SceneData.InstanceDataTileStride * TileId + (ArrayIndex << SceneData.InstanceDataTileSizeLog2) + IdInTile;
}

float4 LoadInstanceSceneDataElement(uint Index)
{
#if SHADER_USES_PRIMITIVE_UBO
	return 0; // FIXME LoadInstanceSceneDataElement for UBO case
#elif USE_GLOBAL_GPU_SCENE_DATA
	return GPUSceneInstanceSceneData[Index];
#elif USE_GPU_SCENE_DATA_RW

#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	return GPUSceneInstanceSceneDataRW[Index];
#else
	return GPUSceneWriter.GPUSceneInstanceSceneDataRW[Index];
#endif
#else
	return Scene.GPUScene.GPUSceneInstanceSceneData[Index];
#endif
}

float4 LoadInstanceSceneDataElement(uint InstanceId, uint ArrayIndex)
{
	return LoadInstanceSceneDataElement(CalcInstanceDataIndex(InstanceId, ArrayIndex));
}

float4 LoadInstancePayloadDataElement(uint Index)
{
#if SHADER_USES_PRIMITIVE_UBO
	return 0; // FIXME LoadInstancePayloadDataElement for UBO case
#elif USE_GLOBAL_GPU_SCENE_DATA
	return GPUSceneInstancePayloadData[Index];
#elif USE_GPU_SCENE_DATA_RW

#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
	return GPUSceneInstancePayloadDataRW[Index];
#else
	return GPUSceneWriter.GPUSceneInstancePayloadDataRW[Index];
#endif
#else
	return Scene.GPUScene.GPUSceneInstancePayloadData[Index];
#endif
}

float4 LoadInstancePayloadExtensionElement(FInstanceSceneData InstanceData, uint Index)
{
	checkSlow(InstanceData.PayloadExtensionOffset != INVALID_INSTANCE_PAYLOAD_OFFSET);
	return LoadInstancePayloadDataElement(InstanceData.PayloadExtensionOffset + Index);
}

float4 LoadInstanceCustomDataElement(FInstanceSceneData SceneData, uint Float4Index)
{
#if ENABLE_PER_INSTANCE_CUSTOM_DATA
	#if SHADER_USES_PRIMITIVE_UBO
		// TODO: atm this is not used anywhere, but will be good to implement as well
		return (float4)0.0f;
	#else
		const uint NumCustomFloat4s = (SceneData.CustomDataCount + 3u) >> 2u;
		if (SceneData.CustomDataOffset != 0xFFFFFFFFu && Float4Index < NumCustomFloat4s)
		{
			return LoadInstancePayloadDataElement(SceneData.CustomDataOffset + Float4Index);
		}
	#endif
#endif
	return (float4)0.0f;
}

float LoadInstanceCustomDataFloat(FInstanceSceneData SceneData, uint FloatIndex)
{
#if ENABLE_PER_INSTANCE_CUSTOM_DATA
	#if SHADER_USES_PRIMITIVE_UBO
		return LoadInstanceCustomDataElementUBO(SceneData.PrimitiveId, FloatIndex, 0.0f);
	#else
		const uint Float4Index = FloatIndex >> 2u;
		const uint ComponentIndex = FloatIndex % 4u;
		const float4 Element = LoadInstanceCustomDataElement(SceneData, Float4Index);
		return Element[ComponentIndex];
	#endif
#else
	return 0.0f;
#endif
}

// [Frisvad 2012, "Building an Orthonormal Basis from a 3D Unit Vector Without Normalization"]
void GetHemiOrthoBasis( inout float3 BasisX, inout float3 BasisY, float3 BasisZ )
{
	float A = 1.0f / ( 1.0f + BasisZ.z );
	float B = -BasisZ.x * BasisZ.y * A;
	BasisX = float3( 1.0f - BasisZ.x * BasisZ.x * A, B, -BasisZ.x );
	BasisY = float3( B, 1.0f - BasisZ.y * BasisZ.y * A, -BasisZ.y );
}

uint4 EncodeScaleAndRotation(float3 Scale, float3x3 Axis)
{
	const uint ExpBits			= 8;
	const uint ExpBias			= ( 1u << (ExpBits - 1) ) - 1;
	const uint SignMantissaBits	= 16;
	const uint SignMantissaMask	= (1u << SignMantissaBits) - 1;
	const uint MantissaBits		= SignMantissaBits - 1;
	const float Sqrt2 			= 1.41421356f;

	uint4 Output;

	// Rotation
	{
		if( Axis[2].z < 0.0f )
		{
			Axis[2] *= -1.0f;
			Scale.z *= -1.0f;
		}

		float2 OctZ = UnitVectorToHemiOctahedron( Axis[2] );

		float3 BasisX, BasisY;
		GetHemiOrthoBasis( BasisX, BasisY, Axis[2] );

		float X = dot(Axis[0], BasisX);
		float Y = dot(Axis[0], BasisY);

		float aX = abs( X );
		float aY = abs( Y );

		bool bSpinIsX = aX < aY;
		float Spin0 = bSpinIsX ? X : Y;
		float Spin1 = bSpinIsX ? Y : X;
		float Sign1 = Spin1 < 0.0f ? -1.0f : 1.0f;
	
		//Axis[0]	*= Sign1;
		Scale.x *= Sign1;
		Spin0	*= Sign1;

		float3 GeneratedY = cross(Axis[2], Axis[0]);
		Scale.y *= dot( Axis[1], GeneratedY ) < 0.0f ? -Sign1 : Sign1;

		// Avoid sign extension in shader by biasing
		Output.x  = (((int)round( OctZ.x * 32767.0f ) + 32768) & 0xFFFF) <<  0;
		Output.x |= (((int)round( OctZ.y * 32767.0f ) + 32768) & 0xFFFF) << 16;

		// NOTE: Masking the bits with `& 0x7FFF` below causes the whole int to be optimized to 0 on some shader platforms.
		// This is okay, as long as Spin0 is in [0, 1], which it should be.
		Output.y  = ((int)round( Spin0 * 16383.0f * Sqrt2 ) + 16384); // & 0x7FFF;
		Output.y |=	bSpinIsX ? (1u << 15) : 0;
	}

	// Scale
	{
		float MaxComponent = max3(abs(Scale.x), abs(Scale.y), abs(Scale.z));
		uint MaxComponentExponent = (asuint(MaxComponent) & 0x7f800000u) >> 23;

		// Need +1 because of losing the implicit leading bit of mantissa
		// TODO assumes ExpBits == 8
		// TODO clamp to expressable range
		uint SharedExp = MaxComponentExponent + 1;

		float ExpScale = asfloat(((127 + ExpBias + MantissaBits - SharedExp) & 0xFFu) << 23);

		if( (uint)round( MaxComponent * ExpScale ) == (1u << MantissaBits) )
		{
			// Mantissa rounded up
			SharedExp++;
			ExpScale *= 0.5f;
		}

		Output.z  = (((int)round( Scale.x * ExpScale ) + (1u << MantissaBits)) & 0xFFFFu) <<  0;
		Output.z |= (((int)round( Scale.y * ExpScale ) + (1u << MantissaBits)) & 0xFFFFu) << 16;
		Output.w  = (((int)round( Scale.z * ExpScale ) + (1u << MantissaBits)) & 0xFFFFu) <<  0;
		Output.w |= SharedExp << 16;		
	}

	return Output;
}

uint4 EncodeScaleAndRotation( float3x3 InTransform )
{
	float3 Scale = {
		length(InTransform[0]),
		length(InTransform[1]),
		length(InTransform[2])
	};
	float3x3 Axis = {
		InTransform[0] / Scale.x,
		InTransform[1] / Scale.y,
		InTransform[2] / Scale.z
	};
	return EncodeScaleAndRotation(Scale, Axis);
}

void EncodeTransform( float4x4 InTransform, inout uint4 OutRotationScale, inout float3 OutTranslation )
{
	OutRotationScale = EncodeScaleAndRotation((float3x3)InTransform);
	OutTranslation = InTransform[3].xyz;
}

float4x4 DecodeTransform( uint4 RotationScale, float3 Translation, inout float3 Scale )
{
	float4x4 M = 0.0;
	M[3].xyz = Translation;
	M[3].w = 1.0;

	// Rotation
	{
		float3 Rotation =
		{
			( RotationScale[0] >>  0 ) & 0xffff,
			( RotationScale[0] >> 16 ) & 0xffff,
			( RotationScale[1] >>  0 ) & 0x7fff
		};

		float2 OctZ  = ( Rotation.xy - 32768 ) * (1.0f / 32767.0f);
		float Spin0  = ( Rotation.z  - 16384 ) * (0.70710678f / 16383.0f);	// rsqrt(2)
		bool bSpinIsX = RotationScale[1] & 0x8000;

		M[2].xyz = HemiOctahedronToUnitVector( OctZ );

		float3 BasisX, BasisY;
		GetHemiOrthoBasis( BasisX, BasisY, M[2].xyz );

		float Spin1 = sqrt( 1.0f - Spin0 * Spin0 );
		float X = bSpinIsX ? Spin0 : Spin1;
		float Y = bSpinIsX ? Spin1 : Spin0;

		M[0].xyz = BasisX * X + BasisY * Y;
		M[1].xyz = cross( M[2].xyz, M[0].xyz );
	}

	// Scale
	{
		const uint SignMantissaBits	= 16;
		const uint SignMantissaMask	= (1u << SignMantissaBits) - 1;
		const uint MantissaBits		= SignMantissaBits - 1;

#if 0
		uint SharedExp = RotationScale[3] >> 22;

		float ExpScale = asfloat( ( SharedExp - MantissaBits ) << 23 );

		int3 Mantissa =
		{
			( RotationScale[2] >> 0 ),
			( RotationScale[2] >> 18 ) | ( RotationScale[3] << 14 ),
			( RotationScale[3] >> 4 )
		};
#else
		uint SharedExp = RotationScale[3] >> 16;

		float ExpScale = asfloat( ( SharedExp - MantissaBits ) << 23 );

		uint3 Mantissa =
		{
			RotationScale[2] >> 0,
			RotationScale[2] >> 16,
			RotationScale[3] >> 0
		};
		
#endif
		Mantissa &= SignMantissaMask;
		Scale = Mantissa;
		Scale -= 1u << MantissaBits;
		Scale *= ExpScale;

		M[0] *= Scale[0];
		M[1] *= Scale[1];
		M[2] *= Scale[2];
	}

	return M;
}

// Helpers to pack/unpack the primitive ID and flags for the specified instance, which are packed together in a uint
void UnpackPrimitiveIdAndInstanceFlags(uint PackedPrimitiveIdAndFlags, inout uint OutPrimitiveId, inout uint OutInstanceFlags)
{
	OutPrimitiveId		= BitFieldExtractU32(PackedPrimitiveIdAndFlags, PRIMITIVE_ID_NUM_BITS,  0);
	OutInstanceFlags	= BitFieldExtractU32(PackedPrimitiveIdAndFlags, INSTANCE_SCENE_DATA_FLAGS_NUM_BITS, PRIMITIVE_ID_NUM_BITS);	
}

uint PackPrimitiveIdAndInstanceFlags(uint PrimitiveId, uint InstanceFlags)
{
	return (PrimitiveId & PRIMITIVE_ID_MASK) | (InstanceFlags << PRIMITIVE_ID_NUM_BITS);
}

void LoadInstancePrimitiveIdAndFlags(uint InstanceId, inout uint OutPrimitiveId, inout uint OutInstanceFlags)
{
	const uint PackedPrimitiveIdAndFlags = asuint(LoadInstanceSceneDataElement(InstanceId, 0).x);
	UnpackPrimitiveIdAndInstanceFlags(PackedPrimitiveIdAndFlags, OutPrimitiveId, OutInstanceFlags);
}

// Helpers to pack/unpack the instance relative ID and custom data count for the specified instance, which are packed together in a uint
void UnpackInstanceRelativeIdAndCustomDataCount(uint PackedRelativeIdAndCustomDataCount, inout uint OutRelativeId, inout uint OutCustomDataCount)
{
	OutRelativeId		= BitFieldExtractU32(PackedRelativeIdAndCustomDataCount, INSTANCE_RELATIVE_ID_NUM_BITS,  0);
	OutCustomDataCount	= BitFieldExtractU32(PackedRelativeIdAndCustomDataCount, INSTANCE_CUSTOM_DATA_COUNT_NUM_BITS, INSTANCE_RELATIVE_ID_NUM_BITS);
}

uint PackInstanceRelativeIdAndCustomDataCount(uint RelativeId, uint CustomDataCount)
{
	return (RelativeId & INSTANCE_RELATIVE_ID_MASK) | (CustomDataCount << INSTANCE_RELATIVE_ID_NUM_BITS);
}

void LoadInstanceRelativeIdAndCustomDataCount(uint InstanceId, inout uint OutPrimitiveId, inout uint OutInstanceFlags)
{
	const uint PackedRelativeIdAndCustomDataCount = asuint(LoadInstanceSceneDataElement(InstanceId, 0).y);
	UnpackInstanceRelativeIdAndCustomDataCount(PackedRelativeIdAndCustomDataCount, OutPrimitiveId, OutInstanceFlags);
}

// Helpers for getting/setting the instance determinant sign from instance data flags
float GetInstanceDeterminantSignFromFlags(uint Flags)
{
	// Scale.x * Scale.y * Scale.z < 0.0 ? -1.0 : 1.0;
	return CondMask(Flags & INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN, -1.0f, 1.0f);
}

void SetInstanceDeterminantSignFlag(float Determinant, inout uint Flags)
{
	if (Determinant < 0.0f)
	{
		Flags |= INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN;
	}
	else
	{
		Flags &= ~INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN;
	}
}

// Determine the offsets into the payload data buffer for the given instance
FInstancePayloadDataOffsets GetInstancePayloadDataOffsets(uint PrimitiveId, uint Flags, uint InstanceRelativeId)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);

	const uint PayloadDataRelativeOffset	= InstanceRelativeId * PrimitiveData.InstancePayloadDataStride;
	const uint PayloadDataGlobalOffset		= PayloadDataRelativeOffset + PrimitiveData.InstancePayloadDataOffset;

	const bool bHasHierarchyOffset			= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET) != 0u;
	const bool bHasLocalBounds				= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS) != 0u;
	const bool bHasDynamicData				= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA) != 0u;
	const bool bHasSkinningData				= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA) != 0u;
	const bool bHasLightShadowUVBias		= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS) != 0u;
	const bool bHasCustomData				= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA) != 0u;
	const bool bHasPayloadExtension			= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION) != 0u;
#if USE_EDITOR_SHADERS
	const bool bHasEditorData				= (Flags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA) != 0u;
#else
	const bool bHasEditorData				= false;
#endif

	uint CurOffset = PayloadDataGlobalOffset;

	// Offsets are in float4s
	FInstancePayloadDataOffsets Offsets;
	Offsets.HierarchyOffset 	= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.EditorData 			= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.LocalBounds 		= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.DynamicData 		= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.SkinningData 		= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.LightShadowUVBias 	= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.PayloadExtension	= INVALID_INSTANCE_PAYLOAD_OFFSET;
	Offsets.CustomData			= INVALID_INSTANCE_PAYLOAD_OFFSET;
	
	// Hierarchy Offset -> float0.x
	if (bHasHierarchyOffset)
	{
		Offsets.HierarchyOffset = CurOffset;
	}
	// SkinningData -> float0.y
	if (bHasSkinningData)
	{
		Offsets.SkinningData = CurOffset;
	}
	// LocalBounds -> float0.zw & float1.xyzw
	if (bHasLocalBounds)
	{
		Offsets.LocalBounds = CurOffset;
	}
	CurOffset += CondMask(bHasLocalBounds, 2u, CondMask(bHasHierarchyOffset || bHasSkinningData, 1u, 0u));

	if (bHasDynamicData)
	{
		Offsets.DynamicData = CurOffset;
		CurOffset += InstanceTransformSizeFloat4Count;
	}

	if (bHasEditorData)
	{
		Offsets.EditorData = CurOffset;
		++CurOffset;
	}

	if (bHasLightShadowUVBias)
	{
		Offsets.LightShadowUVBias = CurOffset;
		++CurOffset;
	}

	if (bHasPayloadExtension)
	{
		Offsets.PayloadExtension = CurOffset;
		CurOffset += PrimitiveData.InstancePayloadExtensionSize;
	}

	if (bHasCustomData)
	{
		Offsets.CustomData = CurOffset;
	}

	return Offsets;
}

void ComputeInstanceDerivedData(inout FInstanceSceneData InstanceData, float3 PositionHigh, float4x4 LocalToRelativeWorld)
{
	//
	// Do not put any load operations here!
	//

#if !PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS && !SHADER_USES_PRIMITIVE_UBO
	// Non-uniform scale must be computed from the transform because it was not already computed when decoding it (see below in GetInstanceSceneData)
	float3 Scale2;
	Scale2.x = length2(LocalToRelativeWorld[0].xyz);
	Scale2.y = length2(LocalToRelativeWorld[1].xyz);
	Scale2.z = length2(LocalToRelativeWorld[2].xyz);

	InstanceData.InvNonUniformScale							= rsqrt(Scale2);
	InstanceData.NonUniformScale.xyz						= Scale2 * InstanceData.InvNonUniformScale;
#endif

	InstanceData.NonUniformScale.w							= max3( InstanceData.NonUniformScale.x, InstanceData.NonUniformScale.y, InstanceData.NonUniformScale.z );
	InstanceData.DeterminantSign							= GetInstanceDeterminantSignFromFlags(InstanceData.Flags);

	float4x4 RelativeWorldToLocal							= LocalToRelativeWorld;
	RelativeWorldToLocal[0].xyz							   *= Pow2(InstanceData.InvNonUniformScale.x);
	RelativeWorldToLocal[1].xyz					           *= Pow2(InstanceData.InvNonUniformScale.y);
	RelativeWorldToLocal[2].xyz					           *= Pow2(InstanceData.InvNonUniformScale.z);
	RelativeWorldToLocal[3].xyz						        = 0.0f;
	RelativeWorldToLocal								    = transpose(RelativeWorldToLocal);
	RelativeWorldToLocal[3].xyz						        = mul(float4(-LocalToRelativeWorld[3].xyz, 0.0f), RelativeWorldToLocal).xyz;
	InstanceData.WorldToLocal								= MakeDFInverseMatrix(PositionHigh, RelativeWorldToLocal);
}

FInstanceSceneData GetInstanceSceneDataInternal(uint InstanceId, bool bCheckValid)
{
	FInstanceSceneData InstanceData = (FInstanceSceneData)0;

	InstanceData.InstanceId = InstanceId;
	//
	// NOTE: When changing the packed data layout, ensure that GPUScene/GPUSceneWriter.ush is kept in sync!
	//       Also, please update the GetInstanceSceneData function in GPUScene.cpp for validation purposes.
	//

	// Only process valid instances
	LoadInstancePrimitiveIdAndFlags(InstanceId, InstanceData.PrimitiveId, InstanceData.Flags);
	InstanceData.ValidInstance = InstanceData.PrimitiveId != INVALID_PRIMITIVE_ID &&
								 ((InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HIDDEN) == 0u);

	BRANCH
	if (!bCheckValid || InstanceData.ValidInstance)
	{
		uint CustomDataCount;
		LoadInstanceRelativeIdAndCustomDataCount(InstanceId, InstanceData.RelativeId, CustomDataCount);

		FInstancePayloadDataOffsets Offsets 	= GetInstancePayloadDataOffsets(InstanceData.PrimitiveId, InstanceData.Flags, InstanceData.RelativeId);
	
	#if ENABLE_PER_INSTANCE_CUSTOM_DATA
		InstanceData.CustomDataCount			= CustomDataCount;
		InstanceData.CustomDataOffset			= Offsets.CustomData;
	#endif
		InstanceData.LastUpdateSceneFrameNumber	= asuint(LoadInstanceSceneDataElement(InstanceId, 0).z);
		InstanceData.RandomID					= LoadInstanceSceneDataElement(InstanceId, 0).w;

		FPrimitiveSceneData PrimitiveData		= GetPrimitiveData(InstanceData.PrimitiveId);

		float3 PositionHigh 					= PrimitiveData.PositionHigh;

	#if PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS
		uint4  RotationScale					= asuint(LoadInstanceSceneDataElement(InstanceId, 1));
		float3 Translation						= LoadInstanceSceneDataElement(InstanceId, 2).xyz;
		float3 Scale = 0;
		float4x4 LocalToRelativeWorld			= DecodeTransform( RotationScale, Translation, Scale );

		InstanceData.NonUniformScale.xyz		= abs(Scale);
		InstanceData.InvNonUniformScale			= rcp(InstanceData.NonUniformScale.xyz);
	#else
		float4x4 LocalToRelativeWorld			= transpose(
													float4x4(
														LoadInstanceSceneDataElement(InstanceId, 1),
														LoadInstanceSceneDataElement(InstanceId, 2),
														LoadInstanceSceneDataElement(InstanceId, 3),
														float4(0.0f, 0.0f, 0.0f, 1.0f)
													));
	#endif


		InstanceData.LocalToWorld	= MakeDFMatrix(PositionHigh, LocalToRelativeWorld);
		ComputeInstanceDerivedData(InstanceData, PositionHigh, LocalToRelativeWorld);

		InstanceData.NaniteRuntimeResourceID		= PrimitiveData.NaniteResourceID;
		InstanceData.NaniteHierarchyOffset			= PrimitiveData.NaniteHierarchyOffset;

		BRANCH
		if (Offsets.HierarchyOffset != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			const uint HierarchyRootOffset = asuint(LoadInstancePayloadDataElement(Offsets.HierarchyOffset)).x;

			// Combine this instance's hierarchy offset with the primitive's root hierarchy offset
			InstanceData.NaniteHierarchyOffset += HierarchyRootOffset;
		}

		InstanceData.SkinningData = 0;
		BRANCH
		if (Offsets.SkinningData != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			InstanceData.SkinningData = asuint(LoadInstancePayloadDataElement(Offsets.SkinningData)).y;
		}

	#if USE_EDITOR_SHADERS
		BRANCH
		if (Offsets.EditorData != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			const uint PackedEditorData = asuint(LoadInstancePayloadDataElement(Offsets.EditorData)).x;
			// Note: .yzw are currently unused

			InstanceData.EditorData.bIsSelected    = (PackedEditorData >> 24u) != 0;
			InstanceData.EditorData.HitProxyPacked =  PackedEditorData & 0x00FFFFFFu;
			InstanceData.EditorData.HitProxyId = UnpackHitProxyId(InstanceData.EditorData.HitProxyPacked);
		}
	#endif

		BRANCH
		if (Offsets.LocalBounds != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			InstanceData.LocalBoundsCenter = float3(LoadInstancePayloadDataElement(Offsets.LocalBounds + 0).zw, LoadInstancePayloadDataElement(Offsets.LocalBounds + 1).x);
			InstanceData.LocalBoundsExtent = LoadInstancePayloadDataElement(Offsets.LocalBounds + 1).yzw;
		}
		else
		{
			InstanceData.LocalBoundsCenter = PrimitiveData.InstanceLocalBoundsCenter;
			InstanceData.LocalBoundsExtent = PrimitiveData.InstanceLocalBoundsExtent;
		}

		BRANCH
		if (Offsets.DynamicData != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
		#if PLATFORM_ALLOW_SCENE_DATA_COMPRESSED_TRANSFORMS
			uint4  PrevRotationScale			= asuint(LoadInstancePayloadDataElement(Offsets.DynamicData + 0));
			float3 PrevTranslation				= LoadInstancePayloadDataElement(Offsets.DynamicData + 1).xyz;
			float3 PrevScale					= 0;
			float4x4 PrevLocalToRelativeWorld	= DecodeTransform(PrevRotationScale, PrevTranslation, PrevScale);
		#else
			float4x4 PrevLocalToRelativeWorld	= transpose(
													float4x4(
														LoadInstancePayloadDataElement(Offsets.DynamicData + 0),
														LoadInstancePayloadDataElement(Offsets.DynamicData + 1),
														LoadInstancePayloadDataElement(Offsets.DynamicData + 2),
														float4(0.0f, 0.0f, 0.0f, 1.0f)
													));
		#endif

			InstanceData.PrevLocalToWorld		= MakeDFMatrix(PositionHigh, PrevLocalToRelativeWorld);
		}
		else
		{
			// Since the instance is not dynamic, it cannot move relative to the primitive. However, the primitive itself can
			// move. So we have to solve for previous Instance->World by taking primitive World->PrevWorld and using it to
			// transform Instance->World into previous world space.
			InstanceData.PrevLocalToWorld = MakeDFMatrix(PositionHigh, mul(LocalToRelativeWorld, PrimitiveData.WorldToPreviousWorld));
		}

	#if 1 //NEEDS_LIGHTMAP_COORDINATE
		BRANCH
		if (Offsets.LightShadowUVBias != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			InstanceData.LightMapAndShadowMapUVBias = LoadInstancePayloadDataElement(Offsets.LightShadowUVBias);
		}
	#endif

		InstanceData.PayloadExtensionOffset = Offsets.PayloadExtension;
		InstanceData.PayloadExtensionSize = PrimitiveData.InstancePayloadExtensionSize;
	}

	return InstanceData;
}

struct FSceneDataIntermediates
{
	uint PrimitiveId;
	uint InstanceId;
	uint ViewIndex;
	uint CullingFlags;
	// Index from which we load the instance info, needed for the 
	uint InstanceIdLoadIndex;
	FInstanceSceneData InstanceData;
	FPrimitiveSceneData Primitive;
};

// Instance culling flags
#define INSTANCE_CULLING_FLAG_EVALUATE_WPO (1u << 0u)

#define INSTANCE_CULLING_FLAGS_DEFAULT (INSTANCE_CULLING_FLAG_EVALUATE_WPO)

// NOTE: Currently limited to 4 bits to fit into instance culling data DWORD
#define INSTANCE_CULLING_FLAGS_NUM_BITS (4u)
#define INSTANCE_CULLING_FLAGS_MASK ((1u << INSTANCE_CULLING_FLAGS_NUM_BITS) - 1u)

uint PackInstanceCullingOutput(uint InstanceId, uint ViewIndex, uint CullingFlags)
{
	// We store the view index (which can be used for instanced stereo or other multi-view in the top four bits)
	// Note: this is an index of views for this render pass, not the view ID in the culling manager.
	return (InstanceId & INSTANCE_ID_MASK) |
		((CullingFlags & INSTANCE_CULLING_FLAGS_MASK) << INSTANCE_ID_NUM_BITS) | 
		(ViewIndex << (INSTANCE_ID_NUM_BITS + INSTANCE_CULLING_FLAGS_NUM_BITS));
}

void UnpackInstanceCullingOutput(uint PackedId, inout uint InstanceId, inout uint ViewIndex, inout uint CullingFlags)
{
	InstanceId = PackedId & INSTANCE_ID_MASK;
	ViewIndex = PackedId >> (INSTANCE_ID_NUM_BITS + INSTANCE_CULLING_FLAGS_NUM_BITS);
	CullingFlags = (PackedId >> INSTANCE_ID_NUM_BITS) & INSTANCE_CULLING_FLAGS_MASK;
}


#if VF_USE_PRIMITIVE_SCENE_DATA && SHADER_USES_PRIMITIVE_UBO
FSceneDataIntermediates GetSceneDataIntermediatesUBO(uint DrawInstanceId)
{
	FSceneDataIntermediates Intermediates = (FSceneDataIntermediates)0;
#if USE_INSTANCE_CULLING
	Intermediates.Primitive = GetPrimitiveDataFromUniformBuffer();
	Intermediates.InstanceData = LoadInstanceDataUBO(DrawInstanceId);
	ComputeInstanceDerivedData(Intermediates.InstanceData, Intermediates.InstanceData.LocalToWorld.PostTranslation, Intermediates.InstanceData.LocalToWorld.M);
#else
	Intermediates.Primitive = LoadPrimitiveDataUBO(DrawInstanceId);
	// Populate instance data from primitive data
	Intermediates.InstanceData.LocalToWorld = Intermediates.Primitive.LocalToWorld;
	Intermediates.InstanceData.PrevLocalToWorld = Intermediates.Primitive.PreviousLocalToWorld;
	Intermediates.InstanceData.WorldToLocal = Intermediates.Primitive.WorldToLocal;
	Intermediates.InstanceData.NonUniformScale = Intermediates.Primitive.NonUniformScale;
	Intermediates.InstanceData.InvNonUniformScale = Intermediates.Primitive.InvNonUniformScale;
	Intermediates.InstanceData.DeterminantSign = GetPrimitive_DeterminantSign_FromFlags(Intermediates.Primitive.Flags);
#endif
	Intermediates.InstanceData.LocalBoundsCenter = Intermediates.Primitive.InstanceLocalBoundsCenter;
	Intermediates.InstanceData.LocalBoundsExtent = Intermediates.Primitive.InstanceLocalBoundsExtent;
	Intermediates.InstanceData.ValidInstance = true;
	Intermediates.InstanceData.PrimitiveId = DrawInstanceId;
	Intermediates.CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;
	Intermediates.ViewIndex = 0U;
	Intermediates.PrimitiveId = DrawInstanceId;
	Intermediates.InstanceId = 0U;
	return Intermediates;
}
#endif

FInstanceSceneData GetInstanceSceneData(uint InstanceId)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && SHADER_USES_PRIMITIVE_UBO
	return GetSceneDataIntermediatesUBO(InstanceId).InstanceData;
#else
	return GetInstanceSceneDataInternal(InstanceId, true);
#endif
}

FInstanceSceneData GetInstanceSceneDataUnchecked(uint InstanceId)
{
#if VF_USE_PRIMITIVE_SCENE_DATA && SHADER_USES_PRIMITIVE_UBO
	return GetSceneDataIntermediatesUBO(InstanceId).InstanceData;
#else
	return GetInstanceSceneDataInternal(InstanceId, false);
#endif
}

/**
 * Load scene data once given the inputs require.
 * InstanceIdOffset - supplied as a vertex stream with 0 instance step rate (constant for all instances)
 * DrawInstanceId - the instance ID (SV_InstanceID) in the current draw
 */
#if VF_USE_PRIMITIVE_SCENE_DATA
FSceneDataIntermediates GetSceneDataIntermediates(uint InstanceIdOffset, uint DrawInstanceId)
{
	FSceneDataIntermediates Intermediates = (FSceneDataIntermediates)0;
#if SHADER_USES_PRIMITIVE_UBO
	Intermediates = GetSceneDataIntermediatesUBO(DrawInstanceId);
#else
	Intermediates.InstanceIdLoadIndex = InstanceIdOffset + DrawInstanceId;
	// GPUCULL_TODO: workaround for the fact that DrawDynamicMeshPassPrivate et al. don't work with GPU-Scene instancing
	//				 instead they mark the top bit in the primitive ID and disable auto instancing such that there is an 1:1:1
	//               drawcmd:primitive:instance. Then we can just look up the primitive and fetch the instance data index.
	// GPUCULL_TODO: Workaround also used by HairStrandsMaterialCommon.ush
	// We mark the PrimitiveID with the top bit in dynamic draw passes
	if ((InstanceIdOffset & VF_TREAT_INSTANCE_ID_OFFSET_AS_PRIMITIVE_ID_FLAG) != 0U)
	{
		// mask off the flag
		uint PrimitiveID = InstanceIdOffset & (VF_TREAT_INSTANCE_ID_OFFSET_AS_PRIMITIVE_ID_FLAG - 1U);
		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveID);
		Intermediates.InstanceId = PrimitiveData.InstanceSceneDataOffset + DrawInstanceId;
		Intermediates.ViewIndex = 0;
		Intermediates.CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;
	}
#if RAYHITGROUPSHADER || RAYTRACING_DYNAMIC_GEOMETRY_CONVERTER
	else
	{
		Intermediates.InstanceId = InstanceIdOffset + DrawInstanceId;
		Intermediates.ViewIndex = 0;
		Intermediates.CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;
	}
// Workaround for Vulkan SPIRV issue when "else" branch is not removed when it should, which leads to higher level code expecting InstanceCulling buffer to be bound.
// See: https://github.com/KhronosGroup/SPIRV-Tools/issues/4902
#elif USE_INSTANCE_CULLING_DATA
	else
	{
		const uint PackedId = InstanceCulling.InstanceIdsBuffer[InstanceIdOffset + DrawInstanceId];
		UnpackInstanceCullingOutput(PackedId, Intermediates.InstanceId, Intermediates.ViewIndex, Intermediates.CullingFlags);
	}
#endif

	Intermediates.InstanceData = GetInstanceSceneData(Intermediates.InstanceId);
	Intermediates.PrimitiveId = Intermediates.InstanceData.PrimitiveId;
	Intermediates.Primitive = GetPrimitiveData(Intermediates.PrimitiveId);
#endif //SHADER_USES_PRIMITIVE_UBO

	return Intermediates;
}
#else
FSceneDataIntermediates GetSceneDataIntermediates()
{
	FSceneDataIntermediates Intermediates = (FSceneDataIntermediates)0;
	// Populate from Primitive uniform buffer
	Intermediates.ViewIndex = 0U;
	Intermediates.PrimitiveId = 0U;
	Intermediates.InstanceId = 0U;
	Intermediates.Primitive = GetPrimitiveDataFromUniformBuffer();
	// Populate instance data from primitive data
	Intermediates.InstanceData.LocalToWorld			= Intermediates.Primitive.LocalToWorld;
	Intermediates.InstanceData.PrevLocalToWorld		= Intermediates.Primitive.PreviousLocalToWorld;
	Intermediates.InstanceData.WorldToLocal			= Intermediates.Primitive.WorldToLocal;
	Intermediates.InstanceData.NonUniformScale		= Intermediates.Primitive.NonUniformScale;
	Intermediates.InstanceData.InvNonUniformScale	= Intermediates.Primitive.InvNonUniformScale;
	Intermediates.InstanceData.DeterminantSign		= GetPrimitive_DeterminantSign_FromFlags(Intermediates.Primitive.Flags);
	Intermediates.InstanceData.LocalBoundsCenter	= (Intermediates.Primitive.LocalObjectBoundsMax + Intermediates.Primitive.LocalObjectBoundsMin) * 0.5f;
	Intermediates.InstanceData.LocalBoundsExtent	= (Intermediates.Primitive.LocalObjectBoundsMax - Intermediates.Primitive.LocalObjectBoundsMin) * 0.5f;
	Intermediates.InstanceData.ValidInstance		= true;
	
	Intermediates.CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;

	return Intermediates;
}
#endif //VF_USE_PRIMITIVE_SCENE_DATA

//
// GPU Lights
//

#if USE_GLOBAL_GPU_SCENE_DATA
ByteAddressBuffer GPUSceneLights;
#endif

#if USE_GLOBAL_GPU_SCENE_DATA
#define GPUSceneLightsBuffer GPUSceneLights
#else
#define GPUSceneLightsBuffer Scene.GPUScene.GPUSceneLightData
#endif

FLightSceneData GetLightSceneData(uint LightId)
{
#if 0
	return GPUSceneLightsBuffer.Load<FLightSceneData>(LightId * SIZEOF_LIGHT_SCENE_DATA);
#else
	FLightSceneData Output;

	uint Offset = LightId * (uint)SIZEOF_LIGHT_SCENE_DATA;

	LoadAndIncrementOffset(Output.WorldPosition.High, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.WorldPosition.Low, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.InvRadius, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.FalloffExponent, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.Direction, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.InverseExposureBlend, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.Tangent, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.SourceRadius, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.SpotAngles, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.SoftSourceRadius, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.SourceLength, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.RectLightBarnCosAngle, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.RectLightBarnLength, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.SpecularScale, GPUSceneLightsBuffer, Offset);
	LoadAndIncrementOffset(Output.DiffuseScale, GPUSceneLightsBuffer, Offset);
#endif

	return Output;
}
