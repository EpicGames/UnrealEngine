// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef PLATFORM_USES_PRIMITIVE_UBO
	#define PLATFORM_USES_PRIMITIVE_UBO 0
#endif

#ifndef PRIMITIVE_HAS_TILEOFFSET_DATA
#error "Missing PRIMITIVE_HAS_TILEOFFSET_DATA"
#endif

float4 LoadPrimitiveDataElementUBO(uint DrawInstanceId, uint DataIdx)
{
#if PLATFORM_USES_PRIMITIVE_UBO
	uint DataOffset = DrawInstanceId * BATCHED_PRIMITIVE_DATA_STRIDE_FLOAT4 + DataIdx;
	return BatchedPrimitive.Data[DataOffset];
#else
	return (float4)0;
#endif
}

FPrimitiveSceneData LoadPrimitiveDataUBO(uint DrawInstanceId)
{
	FPrimitiveSceneData PrimitiveData = (FPrimitiveSceneData)0;
	
	uint DataIdx = 0;
#if ALLOW_STATIC_LIGHTING
	// 2 float4 reservered for pre-computed lighting data
	float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+2);
	PrimitiveData.LightmapDataIndex = asuint(Data.x);
	DataIdx += 3u; 
#endif

	// PositionHigh, Flags
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx + 0);
		PrimitiveData.PositionHigh = Data.xyz;
		PrimitiveData.Flags = asuint(Data.w);
		DataIdx += 1u;
	}

	// LocalToWorld
	{
		float4x4 LocalToRelativeWorld = transpose(float4x4(
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		PrimitiveData.LocalToWorld = MakeDFMatrix4x3(PrimitiveData.PositionHigh, LocalToRelativeWorld);
		DataIdx += 3u;
	}

	// InvNonUniformScale, TODO .w
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		PrimitiveData.InvNonUniformScale = Data.xyz;
		DataIdx += 1u;

		// Computing it from InvScale should be fine?
		PrimitiveData.NonUniformScale.xyz = rcp(PrimitiveData.InvNonUniformScale);
		PrimitiveData.NonUniformScale.w	= max3(PrimitiveData.NonUniformScale.x, PrimitiveData.NonUniformScale.y, PrimitiveData.NonUniformScale.z);
	}

	// ObjectWorldPosition, Radius
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		float4 Data2 = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1);
		#if PRIMITIVE_HAS_TILEOFFSET_DATA
		PrimitiveData.ObjectWorldPositionTO = MakeLWCVector3(Data.xyz, Data2.xyz);
		PrimitiveData.ObjectWorldPosition = DFFromTileOffset(PrimitiveData.ObjectWorldPositionTO);
		#else
		PrimitiveData.ObjectWorldPosition = MakeDFVector3(Data.xyz, Data2.xyz);
		#endif
		PrimitiveData.ObjectRadius = Data.w;
		DataIdx += 2u;
	}

	// ActorWorldPosition, TODO .w
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		float4 Data2 = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1);
		#if PRIMITIVE_HAS_TILEOFFSET_DATA
		PrimitiveData.ActorWorldPositionTO = MakeLWCVector3(Data.xyz, Data2.xyz);
		PrimitiveData.ActorWorldPosition = DFFromTileOffset(PrimitiveData.ActorWorldPositionTO);
		#else
		PrimitiveData.ActorWorldPosition = MakeDFVector3(Data.xyz, Data2.xyz);
		#endif
		DataIdx += 2u;
	}

	// ObjectOrientation, ObjectBoundsX
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		PrimitiveData.ObjectOrientation = Data.xyz;
		PrimitiveData.ObjectBoundsX = Data.w;
		DataIdx += 1u;
	}

	// LocalObjectBoundsMin, ObjectBoundsY
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		PrimitiveData.LocalObjectBoundsMin = Data.xyz;
		PrimitiveData.ObjectBoundsY = Data.w;
		DataIdx += 1u;
	}

	// LocalObjectBoundsMax, ObjectBoundsZ
	{
		float4 Data = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0);
		PrimitiveData.LocalObjectBoundsMax = Data.xyz;
		PrimitiveData.ObjectBoundsZ = Data.w;
		DataIdx += 1u;
	}
	
	// InstanceLocalBounds
	{
		PrimitiveData.InstanceLocalBoundsCenter = (PrimitiveData.LocalObjectBoundsMin + PrimitiveData.LocalObjectBoundsMax) * 0.5f;
		PrimitiveData.InstanceLocalBoundsExtent = PrimitiveData.LocalObjectBoundsMax - PrimitiveData.InstanceLocalBoundsCenter;
	}

	// WorldToLocal, TODO compute from LocalToWorld?
	{
		float4x4 RelativeWorldToLocal = transpose(float4x4(
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		PrimitiveData.WorldToLocal = MakeDFInverseMatrix(PrimitiveData.PositionHigh, RelativeWorldToLocal);
		DataIdx += 3u;
	}

	// PreviousLocalToWorld, TODO make optional
	{
		float4x4 PreviousLocalToRelativeWorld = transpose(float4x4(
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		PrimitiveData.PreviousLocalToWorld = MakeDFMatrix4x3(PrimitiveData.PositionHigh, PreviousLocalToRelativeWorld);
		DataIdx += 3u;
	}

	// PreviousWorldToLocal, TODO make optional
	{
		float4x4 PreviousRelativeWorldToLocal = transpose(float4x4(
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		PrimitiveData.PreviousWorldToLocal = MakeDFInverseMatrix(PrimitiveData.PositionHigh, PreviousRelativeWorldToLocal);
		DataIdx += 3u;
	}

	// Should not be used with regular primitives
	PrimitiveData.WorldToPreviousWorld = float4x4(1,0,0,0, 0,1,0,0, 0,0,1,0, 0,0,0,1);

	// PreSkinnedLocalBounds, TODO.w
	// TODO: this is needed only for skinned meshes
	{
		PrimitiveData.PreSkinnedLocalBoundsMin = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx + 0).xyz;
		PrimitiveData.PreSkinnedLocalBoundsMax = LoadPrimitiveDataElementUBO(DrawInstanceId, DataIdx + 1).xyz;
		DataIdx += 2u;
	}

	// CustomData, TODO make optional
	{
		uint NumCustomData = min(NUM_CUSTOM_PRIMITIVE_DATA, BATCHED_PRIMITIVE_DATA_STRIDE_FLOAT4 - DataIdx); //23u
		UNROLL
		for (uint i = 0; i < NumCustomData; ++i)
		{
			PrimitiveData.CustomPrimitiveData[i] = LoadPrimitiveDataElementUBO(DrawInstanceId,  DataIdx+i);
		}
		DataIdx += NumCustomData;
	}
	return PrimitiveData;
}

float4 LoadInstanceDataElementUBO(uint DrawInstanceId, uint DataIdx)
{
#if PLATFORM_USES_PRIMITIVE_UBO
	uint DataOffset = DrawInstanceId * BATCHED_INSTANCE_DATA_STRIDE_FLOAT4 + DataIdx;
	return BatchedPrimitive.Data[DataOffset];
#else
	return (float4)0;
#endif
}

FInstanceSceneData LoadInstanceDataUBO(uint DrawInstanceId)
{
	FInstanceSceneData InstanceData = (FInstanceSceneData)0;

	uint DataIdx = 0;

	// TilePosition, Flags
	float3 PositionHigh = 0;
	{
		float4 Data = LoadInstanceDataElementUBO(DrawInstanceId, DataIdx + 0);
		PositionHigh = Data.xyz;
		InstanceData.Flags = asuint(Data.w);
		DataIdx += 1u;
	}
		
	// LocalToWorld
	{
		float4x4 LocalToRelativeWorld = transpose(float4x4(
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		InstanceData.LocalToWorld = MakeDFMatrix4x3(PositionHigh, LocalToRelativeWorld);
		DataIdx += 3u;
	}

	// InvNonUniformScale, RandomID
	{
		float4 Data = LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0);
		InstanceData.InvNonUniformScale = Data.xyz;
		InstanceData.RandomID = Data.w;

		// Computing it from InvScale should be fine?
		InstanceData.NonUniformScale.xyz = rcp(InstanceData.InvNonUniformScale);
		InstanceData.NonUniformScale.w	= max3(InstanceData.NonUniformScale.x, InstanceData.NonUniformScale.y, InstanceData.NonUniformScale.z);
		DataIdx += 1u;
	}

	// PreviousLocalToWorld
	{
		float4x4 PreviousLocalToRelativeWorld = transpose(float4x4(
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0),
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+1),
			LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+2),
			float4(0, 0, 0, 1)
		));
		InstanceData.PrevLocalToWorld = MakeDFMatrix4x3(PositionHigh, PreviousLocalToRelativeWorld);
		DataIdx += 3u;
	}
#if ALLOW_STATIC_LIGHTING
	const bool bHasLightShadowUVBias = (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS) != 0u;
	if (bHasLightShadowUVBias)
	{
		InstanceData.LightMapAndShadowMapUVBias = LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0);
	}
	DataIdx += 1u;
#endif

	// SkinningData
	{
		InstanceData.SkinningData = 0;
		const bool bHasSkinningData = (InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA) != 0u;
		if (bHasSkinningData)
		{
			InstanceData.SkinningData = asuint(LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0).x);
		}
		DataIdx += 1u;
	}

	// CustomDataCount, TODO: yzw
	{
		float4 Data = LoadInstanceDataElementUBO(DrawInstanceId, DataIdx+0);
#if ENABLE_PER_INSTANCE_CUSTOM_DATA
		InstanceData.CustomDataCount = asuint(Data.x);
#endif
		DataIdx += 1u;
	}

	return InstanceData;
}

float LoadInstanceCustomDataElementUBO(uint DrawInstanceId, uint CustomDataIdx, float DefaultValue)
{
#if USE_INSTANCE_CULLING
	uint CustomDataFloat4Offset = 8u; // where custom data starts in an Instance UBO
	#if ALLOW_STATIC_LIGHTING
		CustomDataFloat4Offset += 1u; // one float4 reserved for pre-computed lighting
	#endif
	CustomDataFloat4Offset += 1u; // one float4 reserved for skinning data

	const uint Float4Index = (CustomDataIdx >> 2u);
	const uint CustomDataCount = asuint(LoadInstanceDataElementUBO(DrawInstanceId, CustomDataFloat4Offset).x);
	CustomDataFloat4Offset += 1u;
	if (Float4Index < CustomDataCount)
	{
		const float4 Float4Packed = LoadInstanceDataElementUBO(DrawInstanceId, Float4Index + CustomDataFloat4Offset);
		return Float4Packed[CustomDataIdx & 0x3u];
	}
#endif
	return DefaultValue;
}