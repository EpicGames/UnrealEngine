// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#ifndef PLATFORM_USES_PRIMITIVE_UBO
	#define PLATFORM_USES_PRIMITIVE_UBO 0
#endif

#ifndef PRIMITIVE_HAS_TILEOFFSET_DATA
#error "Missing PRIMITIVE_HAS_TILEOFFSET_DATA"
#endif

void WriteDataElementUBO(RWStructuredBuffer<float4> Output, uint Offset, uint DataIdx, float4 Data)
{
#if	PLATFORM_USES_PRIMITIVE_UBO
	uint DataOffset = Offset + DataIdx;
	Output[DataOffset] = Data;
#endif
}

//
// Note: layout must match FBatchedPrimitiveShaderData in PrimitiveUniformShaderParameters.cpp
//
void WritePrimitiveDataUBO(RWStructuredBuffer<float4> Output, uint Offset, FPrimitiveSceneData PrimitiveData, uint MeshLODIndex)
{
	uint DataIdx = 0;

#if ALLOW_STATIC_LIGHTING
	// Pre-computed lighting data 
	{
		float4 LightMapUVScaleBias = float4(1, 1, 0, 0);
		float4 ShadowMapUVScaleBias = float4(1, 1, 0, 0);

		uint LightmapDataIndex = PrimitiveData.LightmapDataIndex + MeshLODIndex;
		if (LightmapDataIndex < GetSceneData().NumLightmapDataItems)
		{
			FLightmapSceneData LightmapData = GetLightmapData(LightmapDataIndex);
			
			LightMapUVScaleBias = LightmapData.LightMapCoordinateScaleBias;
			ShadowMapUVScaleBias = LightmapData.ShadowMapCoordinateScaleBias;
		}
		WriteDataElementUBO(Output, Offset, DataIdx + 0, LightMapUVScaleBias);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, ShadowMapUVScaleBias);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, float4(asfloat(PrimitiveData.LightmapDataIndex), 0.f, 0.f, 0.f));
		DataIdx += 3u;
	}
#endif

	// PositionHigh, Flags
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.LocalToWorld.PostTranslation, asfloat(PrimitiveData.Flags)));
		DataIdx += 1u;
	}

	// LocalToWorld
	{
		float4x4 LocalToWorld = transpose(PrimitiveData.LocalToWorld.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, LocalToWorld[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, LocalToWorld[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, LocalToWorld[2]);
		DataIdx += 3u;
	}

	// InvNonUniformScale, TODO .w
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.InvNonUniformScale, 0.0));
		DataIdx += 1u;
	}

	// ObjectWorldPosition, Radius
	{
		#if PRIMITIVE_HAS_TILEOFFSET_DATA
		float3 Data = PrimitiveData.ObjectWorldPositionTO.Tile;
		float3 Data2 = PrimitiveData.ObjectWorldPositionTO.Offset;
		#else
		float3 Data = PrimitiveData.ObjectWorldPosition.High;
		float3 Data2 = PrimitiveData.ObjectWorldPosition.Low;
		#endif
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(Data, PrimitiveData.ObjectRadius));
		WriteDataElementUBO(Output, Offset, DataIdx + 1, float4(Data2, 0));
		DataIdx += 2u;
	}

	// ActorWorldPosition, TODO .w
	{
		#if PRIMITIVE_HAS_TILEOFFSET_DATA
		float3 Data = PrimitiveData.ActorWorldPositionTO.Tile;
		float3 Data2 = PrimitiveData.ActorWorldPositionTO.Offset;
		#else
		float3 Data = PrimitiveData.ActorWorldPosition.High;
		float3 Data2 = PrimitiveData.ActorWorldPosition.Low;
		#endif
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(Data, 0.0));
		WriteDataElementUBO(Output, Offset, DataIdx + 1, float4(Data2, 0.0));
		DataIdx += 2u;
	}

	// ObjectOrientation, ObjectBoundsX
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.ObjectOrientation, PrimitiveData.ObjectBoundsX));
		DataIdx += 1u;
	}

	// LocalObjectBoundsMin, ObjectBoundsY
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.LocalObjectBoundsMin, PrimitiveData.ObjectBoundsY));
		DataIdx += 1u;
	}

	// LocalObjectBoundsMax, ObjectBoundsZ
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.LocalObjectBoundsMax, PrimitiveData.ObjectBoundsZ));
		DataIdx += 1u;
	}

	// WorldToLocal, TODO compute from LocalToWorld?
	{
		float4x4 RelativeWorldToLocal = transpose(PrimitiveData.WorldToLocal.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, RelativeWorldToLocal[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, RelativeWorldToLocal[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, RelativeWorldToLocal[2]);
		DataIdx += 3u;
	}

	// PreviousLocalToWorld, TODO make optional
	{
		float4x4 PreviousLocalToWorld = transpose(PrimitiveData.PreviousLocalToWorld.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, PreviousLocalToWorld[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, PreviousLocalToWorld[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, PreviousLocalToWorld[2]);
		DataIdx += 3u;
	}

	// PreviousWorldToLocal, TODO make optional
	{
		float4x4 PreviousRelativeWorldToLocal = transpose(PrimitiveData.PreviousWorldToLocal.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, PreviousRelativeWorldToLocal[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, PreviousRelativeWorldToLocal[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, PreviousRelativeWorldToLocal[2]);
		DataIdx += 3u;
	}
	
	// PreSkinnedLocalBounds, TODO.w
	// TODO: this is needed only for skinned meshes
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(PrimitiveData.PreSkinnedLocalBoundsMin, 0.f));
		WriteDataElementUBO(Output, Offset, DataIdx + 1, float4(PrimitiveData.PreSkinnedLocalBoundsMax, 0.f));
		DataIdx += 2u;
	}

	// CustomData, TODO make optional
	{
		uint NumCustomData = min(NUM_CUSTOM_PRIMITIVE_DATA, BATCHED_PRIMITIVE_DATA_STRIDE_FLOAT4 - DataIdx); //23u
		UNROLL
		for (uint i = 0; i < NumCustomData; ++i)
		{
			WriteDataElementUBO(Output, Offset, DataIdx + i, PrimitiveData.CustomPrimitiveData[i]);
		}
		DataIdx += NumCustomData;
	}
}

void WriteInstanceDataUBO(RWStructuredBuffer<float4> Output, uint Offset, uint InstanceId, FInstanceSceneData InstanceData)
{
	uint DataIdx = 0;
	// PositionHigh, Flags
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(InstanceData.LocalToWorld.PostTranslation, asfloat(InstanceData.Flags)));
		DataIdx += 1u;
	}
	// LocalToWorld
	{
		float4x4 LocalToWorld = transpose(InstanceData.LocalToWorld.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, LocalToWorld[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, LocalToWorld[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, LocalToWorld[2]);
		DataIdx += 3u;
	}
	// InvNonUniformScale, RandomID
	{
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(InstanceData.InvNonUniformScale, InstanceData.RandomID));
		DataIdx += 1u;
	}
	// PreviousLocalToWorld
	{
		float4x4 PrevLocalToWorld = transpose(InstanceData.PrevLocalToWorld.M);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, PrevLocalToWorld[0]);
		WriteDataElementUBO(Output, Offset, DataIdx + 1, PrevLocalToWorld[1]);
		WriteDataElementUBO(Output, Offset, DataIdx + 2, PrevLocalToWorld[2]);
		DataIdx += 3u;
	}

	uint CustomDataCount = 0;
	uint InstanceRelativeId = 0;
	LoadInstanceRelativeIdAndCustomDataCount(InstanceId, InstanceRelativeId, CustomDataCount);
	FInstancePayloadDataOffsets Offsets = GetInstancePayloadDataOffsets(InstanceData.PrimitiveId, InstanceData.Flags, InstanceRelativeId);

#if ALLOW_STATIC_LIGHTING
	{
		if (Offsets.LightShadowUVBias != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			float4 LightMapAndShadowMapUVBias = LoadInstancePayloadDataElement(Offsets.LightShadowUVBias);
			WriteDataElementUBO(Output, Offset, DataIdx + 0, LightMapAndShadowMapUVBias);
		}
		DataIdx += 1u;
	}
#endif

	// SkinningData
	{
		if (Offsets.SkinningData != INVALID_INSTANCE_PAYLOAD_OFFSET)
		{
			uint SkinningDataValue = asuint(LoadInstancePayloadDataElement(Offsets.SkinningData)).y;
			WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(asfloat(SkinningDataValue), 0.f, 0.f, 0.f));
		}
		DataIdx += 1u;
	}

	// CustomDataCount, TODO: yzw
	{
		CustomDataCount = min(CustomDataCount, (BATCHED_INSTANCE_DATA_STRIDE_FLOAT4 - DataIdx - 1u) * 4u);
		WriteDataElementUBO(Output, Offset, DataIdx + 0, float4(asfloat(CustomDataCount), 0.f, 0.f, 0.f));
		DataIdx += 1u;
	}

	// TODO: make it optional, doubles instance data size
	{
		// limited space for instance custom data
		InstanceData.CustomDataCount = CustomDataCount;
		InstanceData.CustomDataOffset = Offsets.CustomData;
		uint CustomDataElements = (CustomDataCount + 3u) >> 2u;
		UNROLL
		for (uint i = 0; i < CustomDataElements; ++i)
		{
			float4 CustomData = LoadInstanceCustomDataElement(InstanceData, i);
			WriteDataElementUBO(Output, Offset, DataIdx + i, CustomData);
		}
	}
}