// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneTexturesCommon.ush"
#include "DeferredShadingCommon.ush"

// this code has been moved to DeferredShadingCommon.ush