// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneTexturesCommon.ush
=============================================================================*/

#pragma once

#include "Common.ush"

// Return far plane when scene textures are disabled in order not to break depthfade
#define SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE 1e6

// Only reference SceneTexturesStruct uniform buffer if SHADING_PATH_DEFERRED
#if SHADING_PATH_DEFERRED

#if !SUPPORTS_INDEPENDENT_SAMPLERS
	#error The deferred shading path only supports independent samplers.
#endif

#define SceneTexturesStruct_SceneColorTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_SceneDepthTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_ScenePartialDepthTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_CustomDepthTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferATextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferBTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferCTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferDTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferETextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferFTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_GBufferVelocityTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_ScreenSpaceAOTextureSampler SceneTexturesStruct.PointClampSampler
#define SceneTexturesStruct_SGGXTextureSampler SceneTexturesStruct.PointClampSampler

float3 CalcSceneColor(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return float3(0.0f,0.0f,0.0f);
#else
	return Texture2DSampleLevel(SceneTexturesStruct.SceneColorTexture, SceneTexturesStruct_SceneColorTextureSampler, ScreenUV, 0).rgb;
#endif
}

float4 CalcFullSceneColor(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return float4(0.0f, 0.0f, 0.0f, 0.0f);
#else
	return Texture2DSample(SceneTexturesStruct.SceneColorTexture, SceneTexturesStruct_SceneColorTextureSampler,ScreenUV);
#endif
}

#ifdef PATH_TRACING
static float PathTracerSceneDepth = 0.0;
#endif

/** Fetches device depth and converts to linear. */
float CalcSceneDepth(float2 ScreenUV)
{
#if PATH_TRACING
	return PathTracerSceneDepth;
#elif SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return ConvertFromDeviceZ(Texture2DSampleLevel(SceneTexturesStruct.SceneDepthTexture, SceneTexturesStruct_SceneDepthTextureSampler, ScreenUV, 0).r);
#endif
}

/** Returns scene color in rgb, depth in alpha. */
float4 CalcSceneColorAndDepth( float2 ScreenUV )
{
	return float4(CalcSceneColor(ScreenUV), CalcSceneDepth(ScreenUV));
}

/** Returns DeviceZ which is the z value stored in the depth buffer. */
float LookupDeviceZ( float2 ScreenUV )
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	// native Depth buffer lookup
	return Texture2DSampleLevel(SceneTexturesStruct.SceneDepthTexture, SceneTexturesStruct_SceneDepthTextureSampler, ScreenUV, 0).r;
#endif
}

/** Returns DeviceZ which is the z value stored in the depth buffer. */
float LookupDeviceZ(uint2 PixelPos)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return SceneTexturesStruct.SceneDepthTexture.Load(int3(PixelPos, 0)).r;
#endif
}

/** Returns 4 DeviceZ values, which is the z value stored in the depth buffer. */
float4 GatherDeviceZ( float2 ScreenUV )
{
#if SCENE_TEXTURES_DISABLED
	return (float4)SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return SceneTexturesStruct.SceneDepthTexture.GatherRed(GlobalBilinearClampedSampler, ScreenUV);
#endif
}

/** Returns clip space W, which is world space distance along the View Z axis. */
float CalcSceneDepth(uint2 PixelPos)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	float DeviceZ = SceneTexturesStruct.SceneDepthTexture.Load(int3(PixelPos, 0)).r;

	// Fetch the depth buffer Z / W value, solve for W
	return ConvertFromDeviceZ(DeviceZ);
#endif
}

// gets 4 nearby SceneDepth values for one UV value, useful for depth downsample, uses Gather() if possible
float4 GatherSceneDepth(float2 UV, float2 InvBufferSize)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return GatherDepth(SceneTexturesStruct.SceneDepthTexture, UV);
#endif
}

/** Fetches custom device depth and converts to linear. */
float CalcSceneCustomDepth(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return ConvertFromDeviceZ(Texture2DSampleLevel(SceneTexturesStruct.CustomDepthTexture, SceneTexturesStruct_CustomDepthTextureSampler, ScreenUV, 0).r);
#endif
}

uint CalcSceneCustomStencil(uint2 PixelPos)
{
#if SCENE_TEXTURES_DISABLED
	return 0;
#else
	return SceneTexturesStruct.CustomStencilTexture.Load(uint3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
#endif
}

float CalcSceneAO(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return 1.0f;
#else
	return Texture2DSampleLevel(SceneTexturesStruct.ScreenSpaceAOTexture, SceneTexturesStruct_ScreenSpaceAOTextureSampler, ScreenUV, 0).r;
#endif
}

#endif // SHADING_PATH_DEFERRED

// Only reference MobileSceneTextures uniform buffer if SHADING_PATH_MOBILE
#if SHADING_PATH_MOBILE

float3 CalcSceneColor(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return float3(0.0f, 0.0f, 0.0f);
#else
	return Texture2DSampleLevel(MobileSceneTextures.SceneColorTexture, MobileSceneTextures.SceneColorTextureSampler, ScreenUV, 0).rgb;
#endif
}

/** return all channels of the scene lighting texture */
float4 CalcFullSceneColor(float2 ScreenUV)
{
#if SCENE_TEXTURES_DISABLED
	return float4(0.0f, 0.0f, 0.0f, FarDepthValue);
#else
	return Texture2DSample(MobileSceneTextures.SceneColorTexture, MobileSceneTextures.SceneColorTextureSampler,ScreenUV);
#endif
}

#ifndef MOBILE_DEFERRED_SHADING
	#define MOBILE_DEFERRED_SHADING 0
#endif

#ifndef POST_PROCESS_AR_PASSTHROUGH
	#define	POST_PROCESS_AR_PASSTHROUGH 0
#endif

#ifndef IS_MOBILE_DEFERREDSHADING_SUBPASS
	#define IS_MOBILE_DEFERREDSHADING_SUBPASS 0
#endif

#ifndef IS_MOBILE_DEPTHREAD_SUBPASS
	#define IS_MOBILE_DEPTHREAD_SUBPASS IS_MOBILE_DEFERREDSHADING_SUBPASS
#endif

// Special intrinsic to read from the current depth buffer
#define MOBILE_DEPTHFECTH (IS_MOBILE_DEPTHREAD_SUBPASS || POST_PROCESS_AR_PASSTHROUGH) && PIXELSHADER && ALLOW_FRAMEBUFFER_FETCH

#if MOBILE_DEPTHFECTH && VULKAN_PROFILE
#include "/Engine/Public/Platform/Vulkan/VulkanSubpassSupport.ush"
#elif MOBILE_DEPTHFECTH && COMPILER_GLSL_ES3_1
#include "/Engine/Public/Platform/GL/GLSubpassSupport.ush"
#endif


/** Returns DeviceZ which is the z value stored in the depth buffer. */
float LookupDeviceZ( float2 ScreenUV, float ArrayIndex )
{
#if	SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#elif FORCE_DEPTH_TEXTURE_READS || PLATFORM_NEEDS_DEPTH_TEXTURE_READS
	// native Depth buffer lookup
	#if MOBILE_MULTI_VIEW
		return MobileSceneTextures.SceneDepthTextureArray.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, float3(ScreenUV.xy,ArrayIndex), 0).r;
	#else
		return MobileSceneTextures.SceneDepthTexture.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, ScreenUV, 0).r;
	#endif
#elif MOBILE_DEPTHFECTH && COMPILER_GLSL_ES3_1
	return DepthbufferFetchES2();
#elif MOBILE_DEPTHFECTH && VULKAN_PROFILE
	return VulkanSubpassDepthFetch();
#elif MOBILE_DEPTHFECTH && (METAL_ES3_1_PROFILE && !MAC) && USE_SCENE_DEPTH_AUX
	return DepthbufferFetchES2();
#elif (USE_SCENE_DEPTH_AUX && !MOBILE_DEFERRED_SHADING)
	// SceneDepth texture is discarded after BasePass (with forward shading) 
	// instead fetch DeviceZ from SceneDepthAuxTexture
	#if MOBILE_MULTI_VIEW
		return MobileSceneTextures.SceneDepthAuxTextureArray.SampleLevel(MobileSceneTextures.SceneDepthAuxTextureSampler, float3(ScreenUV.xy,ArrayIndex), 0).r;
	#else
		return MobileSceneTextures.SceneDepthAuxTexture.SampleLevel(MobileSceneTextures.SceneDepthAuxTextureSampler, ScreenUV, 0).r;
	#endif
#else
	// native Depth buffer lookup
	#if MOBILE_MULTI_VIEW
		return MobileSceneTextures.SceneDepthTextureArray.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, float3(ScreenUV.xy, ArrayIndex), 0).r;
	#else
		return MobileSceneTextures.SceneDepthTexture.SampleLevel(MobileSceneTextures.SceneDepthTextureSampler, ScreenUV, 0).r;
	#endif
#endif
}

/** Returns DeviceZ which is the z value stored in the depth buffer. */
float LookupDeviceZ( float2 ScreenUV )
{
#if MOBILE_MULTI_VIEW
	return LookupDeviceZ(ScreenUV, ResolvedView.StereoPassIndex);
#else
	return LookupDeviceZ(ScreenUV, 0);
#endif
}

/** Returns DeviceZ which is the z value stored in the depth buffer. */
float LookupDeviceZ(uint2 PixelPos)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return MobileSceneTextures.SceneDepthTexture.Load(int3(PixelPos, 0)).r;
#endif
}

/** Returns 4 DeviceZ values, which is the z value stored in the depth buffer. */
float4 GatherDeviceZ( float2 ScreenUV )
{
#if	SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#elif FORCE_DEPTH_TEXTURE_READS || PLATFORM_NEEDS_DEPTH_TEXTURE_READS
	// native Depth buffer lookup
	return MobileSceneTextures.SceneDepthTexture.GatherRed(MobileSceneTextures.SceneDepthTextureSampler, ScreenUV);
#elif (USE_SCENE_DEPTH_AUX && !MOBILE_DEFERRED_SHADING)
	// SceneDepth texture is discarded after BasePass (with forward shading) 
	// instead fetch DeviceZ from SceneDepthAuxTexture
	return MobileSceneTextures.SceneDepthAuxTexture.GatherRed(MobileSceneTextures.SceneDepthAuxTextureSampler, ScreenUV);
#else
	// native Depth buffer lookup
	return MobileSceneTextures.SceneDepthTexture.GatherRed(MobileSceneTextures.SceneDepthTextureSampler, ScreenUV);
#endif
}

/** Returns clip space W, which is world space distance along the View Z axis. Note if you need DeviceZ LookupDeviceZ() is the faster option */
float CalcSceneDepth(float2 ScreenUV, float ArrayIndex)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	#if MOBILE_MULTI_VIEW
			return ConvertFromDeviceZ(LookupDeviceZ(ScreenUV, ArrayIndex));
	#else
			return ConvertFromDeviceZ(LookupDeviceZ(ScreenUV));
	#endif
#endif
}

float CalcSceneDepth(float2 ScreenUV)
{
#if MOBILE_MULTI_VIEW
	return CalcSceneDepth(ScreenUV, ResolvedView.StereoPassIndex);
#else
	return CalcSceneDepth(ScreenUV, 0);
#endif
}

/** Returns clip space W, which is world space distance along the View Z axis. */
float CalcSceneDepth(uint2 PixelPos)
{
#if SCENE_TEXTURES_DISABLED
	return SCENE_TEXTURES_DISABLED_SCENE_DEPTH_VALUE;
#else
	return ConvertFromDeviceZ(LookupDeviceZ(PixelPos));
#endif
}

#endif // SHADING_PATH_MOBILE
