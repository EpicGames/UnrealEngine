// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

#if MOBILE_MULTI_VIEW
Texture2DArray InputTexture;
#else
Texture2D InputTexture;
#endif
SamplerState InputSampler;

void ScreenPassVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	in FStereoVSInput StereoInput,
	out noperspective float4 OutUVAndScreenPos : TEXCOORD0,
	out FStereoVSOutput StereoOutput,
	out float4 OutPosition : SV_POSITION
)
{
	ScreenPassStereoOutputSetupVS(StereoInput, StereoOutput);
	DrawRectangle(InPosition, InTexCoord, OutPosition, OutUVAndScreenPos);
}

float4 CopyRectPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	in FStereoPSInput StereoInput
	) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
#if MOBILE_MULTI_VIEW
	return InputTexture.SampleLevel(InputSampler, float3(UV, GetEyeIndex(StereoInput)), 0);
#else
	return InputTexture.SampleLevel(InputSampler, UV, 0);
#endif
}