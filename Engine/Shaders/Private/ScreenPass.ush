// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define SCREEN_PASS_STRUCT_MEMBER(StructName, MemberType, MemberName) MemberType StructName##_##MemberName;

#define SCREEN_PASS_TEXTURE_VIEWPORT(StructName) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, Extent) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, ExtentInverse) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, ScreenPosToViewportScale) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, ScreenPosToViewportBias) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, uint2, ViewportMin) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, uint2, ViewportMax) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, ViewportSize) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, ViewportSizeInverse) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportMin) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportMax) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportSize) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportSizeInverse) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportBilinearMin) \
	SCREEN_PASS_STRUCT_MEMBER(StructName, float2, UVViewportBilinearMax)

// Substitute for structure abstraction until HLSLCC is gone
#define FScreenTransform float4

float2 ApplyScreenTransform(float2 PInA, FScreenTransform AToB)
{
	return PInA * AToB.xy + AToB.zw;
}

#if PLATFORM_SUPPORTS_REAL_TYPES

half2 ApplyScreenTransform(half2 PInA, FScreenTransform AToB)
{
	return PInA * half2(AToB.xy) + half2(AToB.zw);
}

#endif
