// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenPixelShader.usf: Filter pixel shader source.
=============================================================================*/

#include "Common.ush"

#ifndef SCREEN_PS_FROM_SLICE0
	#define SCREEN_PS_FROM_SLICE0   0
#endif

#ifndef SCREEN_PS_UNWRAP_SLICES
	#define SCREEN_PS_UNWRAP_SLICES 0
#endif

#ifndef SCREEN_PS_SINGLE_SLICE
	#define SCREEN_PS_SINGLE_SLICE 0
#endif

#if SCREEN_PS_SINGLE_SLICE
	uint ArraySlice;
	#define ARRAY_SLICE ArraySlice
#else
	// if source is an array, we only need the first slice
	#define ARRAY_SLICE 0
#endif

#if (SCREEN_PS_FROM_SLICE0 || SCREEN_PS_UNWRAP_SLICES || SCREEN_PS_SINGLE_SLICE)
	Texture2DArray InTexture;

	#define TexOrArr2DSample(Tex, S, UV)			Tex.Sample(S, float3(UV.x, UV.y, ARRAY_SLICE))
	#define TexOrArr2DSampleLevel(Tex, S, UV, L)	Tex.SampleLevel(S, float3(UV.x, UV.y, ARRAY_SLICE), L)
#else
	Texture2D InTexture;

	#define TexOrArr2DSample(Tex, S, UV)			Texture2DSample(Tex, S, UV)
	#define TexOrArr2DSampleLevel(Tex, S, UV, L)	Texture2DSampleLevel(Tex, S, UV, L)
#endif

SamplerState InTextureSampler;
int MipLevel;

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TexOrArr2DSample(InTexture, InTextureSampler, Input.UV);
}

#if SCREEN_PS_UNWRAP_SLICES
void MainUnwrap(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	// for the first half of the screen, sample from slice 0, for the second, from slice 1
	OutColor = InTexture.Sample(InTextureSampler, float3(frac(Input.UV.x * 2.0), Input.UV.y, (float)((int)(2 * Input.UV.x))) );
}
#endif

void MainInvertAlpha(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TexOrArr2DSample(InTexture, InTextureSampler, Input.UV);
	OutColor.a = 1.f - OutColor.a;
}

void MainsRGBSource(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TexOrArr2DSample(InTexture, InTextureSampler, Input.UV);
	OutColor.rgb = pow( OutColor.rgb, 1.0f / 2.2f );
}

void MainMipLevel(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TexOrArr2DSampleLevel(InTexture, InTextureSampler, Input.UV, MipLevel);
}

void MainsRGBSourceMipLevel(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TexOrArr2DSampleLevel(InTexture, InTextureSampler, Input.UV, MipLevel);
	OutColor.rgb = pow( OutColor.rgb, 1.0f / 2.2f );
}
