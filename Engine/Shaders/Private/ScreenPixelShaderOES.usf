// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenPixelShaderOES.usf: Filter pixel shader source.
=============================================================================*/

#include "Common.ush"

#if COMPILER_GLSL_ES3_1
TextureExternal InTexture;
#else
Texture2D InTexture;
#endif

SamplerState InTextureSampler;

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
    OutColor = InTexture.Sample(InTextureSampler, Input.UV);
}
