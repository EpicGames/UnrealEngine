// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------- INCLUDE

#include "../Common.ush"
#include "../MonteCarlo.ush"
#include "../BRDF.ush"
#include "../DeferredShadingCommon.ush"
#include "../CommonViewUniformBuffer.ush"


//------------------------------------------------------- FUNCTIONS

float ComputeDenoisingCoc(float DistanceCameraToPixel, float ClosestHitDistance)
{
	return ClosestHitDistance / (DistanceCameraToPixel + ClosestHitDistance);
}

float ComputeHitDistanceFromDenoisingCoc(float DistanceCameraToPixel, float MaxCoc)
{
	return MaxCoc * DistanceCameraToPixel / (1.0 - MaxCoc);
}

void ComputeSpecularLobeAngles(float3 V, float3 N, float Roughness, out float OutMajorRoughnessLobeAngle, out float OutMinorRoughnessLobeAngle)
{
	// Measure the angle that only truncate 10% of D_GGX's tail
	const float E = 0.5;

	// 4 points, uniformly around the disk, 
	const float2 ArrayE[4] = {
		float2(0.00, E),
		float2(0.25, E),
		float2(0.50, E),
		float2(0.75, E),
	};
		
	float a = Pow2(Roughness);
	
	float3x3 TangentBasis = GetTangentBasis(N);
	float3 TangentV = mul(TangentBasis, V);
		
	// TODO(Denoiser): optimise using lobe symetry along anisotropy major axis.
	float3 TangentL[4];
	UNROLL_N(4)
	for (uint i = 0; i < 4; i++)
	{
		// TODO(Denoiser): optimise VALU here.
		float3 TangentH = ImportanceSampleVisibleGGX(ArrayE[i], a, TangentV).xyz;
		TangentL[i] = 2 * dot(TangentV, TangentH) * TangentH - TangentV;
	}
	
	OutMajorRoughnessLobeAngle = acos(dot(TangentL[2], TangentL[3]));
	OutMinorRoughnessLobeAngle = acos(dot(TangentL[0], TangentL[1]));
}

/** Return the size of the bluring radius caused by a lobe in ViewportUV.x space from radian opening angle. */
float LobeAngleToViewportRadius(float LobeAngle)
{
	return tan(0.5 * LobeAngle) * GetCotanHalfFieldOfView().x;
}
