// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SSDCommon.ush: All the code commonly shared between uniform buffers.
=============================================================================*/

#pragma once

#if !defined(SPHERICAL_HARMONIC_ORDER)
	#define SPHERICAL_HARMONIC_ORDER 0
#endif

#include "../Common.ush"
#include "../SceneTextureParameters.ush"
#include "../ColorSpace.ush"
#include "../SphericalGaussian.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"
#include "/Engine/Private/Substrate/Substrate.ush"

#if SPHERICAL_HARMONIC_ORDER > 0
	#include "../SHCommon.ush"
#endif

#include "SSDPublic.ush"
#include "SSDDefinitions.ush"
#include "SSDMetadata.ush"


//------------------------------------------------------- SHARED CONSTANTS DEPENDING ON CONFIG OF SHADER

#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
	#define TARGETED_SAMPLE_COUNT 1024

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
	// A gray scale valued encoded into a 16bit float only have 10bits mantissa.
	#define TARGETED_SAMPLE_COUNT 1024

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO
	#define TARGETED_SAMPLE_COUNT 256

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
	// Polychromatic penumbra is encded into 16bit float that only have 10bits mantissa.
	#define TARGETED_SAMPLE_COUNT 1024

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
	// Indirect diffuse requires really large spatial kernel to clean the noise.
	#define TARGETED_SAMPLE_COUNT 4096

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC
	#define TARGETED_SAMPLE_COUNT 128

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
	#define TARGETED_SAMPLE_COUNT 128
	// #define CONFIG_METADATA_BUFFER_LAYOUT METADATA_BUFFER_LAYOUT_DEPTH_VIEWNORMAL // TODO Guillaume:
	//#define CONFIG_USE_VIEW_SPACE 1

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
	#define TARGETED_SAMPLE_COUNT 32
	#define CONFIG_METADATA_BUFFER_LAYOUT METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL

#else
	#error Unconfigured TARGETED_SAMPLE_COUNT

#endif

#ifndef CONFIG_METADATA_BUFFER_LAYOUT
	#define CONFIG_METADATA_BUFFER_LAYOUT METADATA_BUFFER_LAYOUT_DISABLED
#endif

#ifndef CONFIG_USE_VIEW_SPACE
	#define CONFIG_USE_VIEW_SPACE 0
#endif

#if CONFIG_USE_VIEW_SPACE && CONFIG_METADATA_BUFFER_LAYOUT != METADATA_BUFFER_LAYOUT_DEPTH_VIEWNORMAL
	#error The all point of using view space is to avoid matrix view -> world computation of the metadata.
#endif


//------------------------------------------------------- SHARED SHADER PARAMETERS

uint2 ViewportMin;
uint2 ViewportMax;
float4 ThreadIdToBufferUV;
float4 DenoiserBufferSizeAndInvSize;
float4 DenoiserBufferBilinearUVMinMax;
float2 BufferUVToOutputPixelPosition;
float4 SceneBufferUVToScreenPosition;
float4x4 ScreenToView;
float2 BufferUVBilinearCorrection;


float4 PrevSceneBufferUVToScreenPosition;

Texture2D PrevDepthBuffer;
Texture2D PrevGBufferA;
Texture2D PrevGBufferB;
// There is intentionally no PrevScreenToView, because the all point of is to avoid full matrix computation in inner loops.

#if CONFIG_METADATA_BUFFER_LAYOUT == METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL

Texture2D<float> CompressedMetadata_0;
Texture2D<uint> CompressedMetadata_1;

Texture2D<float> PrevCompressedMetadata_0;
Texture2D<uint> PrevCompressedMetadata_1;

#else

Texture2D<uint> CompressedMetadata_0;
Texture2D<uint> CompressedMetadata_1;

Texture2D<uint> PrevCompressedMetadata_0;
Texture2D<uint> PrevCompressedMetadata_1;

#endif

#if DEBUG_OUTPUT
RWTexture2D<float4> DebugOutput;
#endif


//------------------------------------------------------- SHARED FUNCTION

float SafeRcp(float x)
{
	if (x > 0)
		return rcp(x);
	return 0;
}

#if CONFIG_USE_VIEW_SPACE
float4x4 GetScreenToViewDistanceMatrix()
{
	float4x4 LocalScreenToView = ScreenToView;

	// Intentionally nullout some values, so the compiler can remove some of the VALU, that is all the point of CONFIG_USE_VIEW_SPACE.
	// TODO(Denoiser): off axis projection.
	{
		LocalScreenToView[0][1] = 0.0;
		LocalScreenToView[0][2] = 0.0;
		LocalScreenToView[0][3] = 0.0;
		
		LocalScreenToView[1][0] = 0.0;
		LocalScreenToView[1][2] = 0.0;
		LocalScreenToView[1][3] = 0.0;
		
		LocalScreenToView[2][0] = 0.0;
		LocalScreenToView[2][1] = 0.0;
		LocalScreenToView[2][2] = 1.0;
		LocalScreenToView[2][3] = 0.0;

		LocalScreenToView[3][0] = 0.0;
		LocalScreenToView[3][1] = 0.0;
		LocalScreenToView[3][2] = 0.0;
		LocalScreenToView[3][3] = 1.0;
	}

	return LocalScreenToView;
}
#endif

//------------------------------------------------------- SPHERICAL HARMONIC INTERFACE

#if SPHERICAL_HARMONIC_ORDER == 0
	// NOP

#elif SPHERICAL_HARMONIC_ORDER == 2
#define FSSDSphericalHarmonic FTwoBandSHVector
#define FSSDSphericalHarmonicRGB FTwoBandSHVectorRGB
#define SSDAddSH AddSH

#elif SPHERICAL_HARMONIC_ORDER == 3
#define FSSDSphericalHarmonic FThreeBandSHVector
#define FSSDSphericalHarmonicRGB FThreeBandSHVectorRGB
#define SSDAddSH AddSH3

#else
	#error Unknown spherical harmonic configuration.

#endif


/** Compute the anisotropy value. The lower the value is, the more anysotropic the sample is. */
float ComputeAnisotropyInvFactor(FSSDSampleSceneInfos SceneMetadata)
{
	if (!CONFIG_NORMAL_ANISOTROPY)
	{
		return 1;
	}

	float Anysotropy = abs(dot(GetWorldNormal(SceneMetadata), View.ViewForward));

	return max(Anysotropy, rcp(8));
}

/** Returns the radius of a pixel in world space. */
float ComputeWorldBluringRadiusCausedByPixelSize(FSSDSampleSceneInfos SceneMetadata)
{
	// Should be multiplied 0.5* for the diameter to radius, and by 2.0 because GetTanHalfFieldOfView() cover only half of the pixels.
	return GetDepthPixelRadiusForProjectionType(GetWorldDepth(SceneMetadata));
}

/** Convert to bilateral world distance. */
float WorldBluringRadiusToBilateralWorldDistance(float WorldBluringRadius)
{
	float Multiplier = 1;

	// The distance between two pixel is 2 times the radius of the pixel.
	Multiplier *= 2;

	// Need to take into account the furthearest pixel of 3x3 neighborhood.
	Multiplier *= sqrt(2);

	// Can take into account how much the signal is getting stored into higher resolution.
	Multiplier *= BLURING_ESTIMATION_MULTIPLIER;

	return WorldBluringRadius * Multiplier;
}

uint2 BufferUVToBufferPixelCoord(float2 SceneBufferUV)
{
	return uint2(SceneBufferUV * BufferUVToOutputPixelPosition);
}

float2 DenoiserBufferUVToScreenPosition(float2 SceneBufferUV)
{
	return SceneBufferUV * SceneBufferUVToScreenPosition.xy + SceneBufferUVToScreenPosition.zw;
}

FSSDCompressedSceneInfos MaterialToCompressedSceneMetadata(float Depth, float3 WorldNormal, float Roughness, uint ShadingID)
{
	FSSDCompressedSceneInfos CompressedMetadata = CreateCompressedSceneInfos();

	CompressedMetadata.VGPR[0] = asuint(Depth);
	CompressedMetadata.VGPR[1] = asuint(WorldNormal.x);
	CompressedMetadata.VGPR[2] = asuint(WorldNormal.y);
	CompressedMetadata.VGPR[3] = asuint(WorldNormal.z);
	CompressedMetadata.VGPR[4] = asuint(Roughness);
	CompressedMetadata.VGPR[5] = ShadingID;

	return CompressedMetadata;
}

FSSDCompressedSceneInfos SampleCompressedSceneMetadata(
	const bool bPrevFrame,
	float2 BufferUV, uint2 PixelCoord)
#if CONFIG_METADATA_BUFFER_LAYOUT == METADATA_BUFFER_LAYOUT_DISABLED
{
  #if SUBSTRATE_GBUFFER_FORMAT==1
	float SceneDepth = 0;
	if (bPrevFrame)
	{
		SceneDepth = ConvertFromDeviceZ(PrevDepthBuffer.SampleLevel(GlobalPointClampedSampler, BufferUV, 0).r);
	}
	else
	{
		SceneDepth = ConvertFromDeviceZ(SampleDeviceZFromSceneTextures(BufferUV));
	}

	// SUBSTRATE_TODO: Manage bPrevFrame
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelCoord, 0)));
	return MaterialToCompressedSceneMetadata(SceneDepth, TopLayerData.WorldNormal, TopLayerData.Roughness, IsSubstrateMaterial(TopLayerData) ? SHADINGMODELID_SUBSTRATE : SHADINGMODELID_UNLIT);
  #else // SUBSTRATE_GBUFFER_FORMAT==1
	FGBufferData GBufferData;
	if (bPrevFrame)
	{
		float DeviceZ = PrevDepthBuffer.SampleLevel(GlobalPointClampedSampler, BufferUV, 0).r;

		uint CustomStencil = 0;
		float CustomNativeDepth = 0;

		float SceneDepth = ConvertFromDeviceZ(DeviceZ);

		float4 GBufferA = PrevGBufferA.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
		float4 GBufferB = PrevGBufferB.SampleLevel(GlobalPointClampedSampler, BufferUV, 0);
		float4 GBufferC = 0.0;
		float4 GBufferD = 0.0;
		float4 GBufferE = 0.0;
		float4 GBufferF = 0.5f;
		float4 GBufferVelocity = 0.0;

		bool bGetNormalizedNormal = false;

		GBufferData = DecodeGBufferData(
			GBufferA, GBufferB, GBufferC, GBufferD, GBufferE, GBufferF, GBufferVelocity,
			CustomNativeDepth, CustomStencil, SceneDepth, bGetNormalizedNormal, CheckerFromSceneColorUV(BufferUV));
	}
	else
	{
		GBufferData = GetGBufferDataFromSceneTextures(BufferUV);	
	}
	return MaterialToCompressedSceneMetadata(GBufferData.Depth, GBufferData.WorldNormal, GBufferData.Roughness, GBufferData.ShadingModelID);
  #endif  // SUBSTRATE_GBUFFER_FORMAT==1
}
#elif CONFIG_METADATA_BUFFER_LAYOUT == METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL
{
	FSSDCompressedSceneInfos CompressedMetadata = CreateCompressedSceneInfos();

	int3 Coord = int3(PixelCoord, /* MipLevel = */ 0);

	if (bPrevFrame)
	{
		CompressedMetadata.VGPR[0] = asuint(PrevCompressedMetadata_0.Load(Coord));
		CompressedMetadata.VGPR[1] = PrevCompressedMetadata_1.Load(Coord);
	}
	else
	{
		CompressedMetadata.VGPR[0] = asuint(CompressedMetadata_0.Load(Coord));
		CompressedMetadata.VGPR[1] = CompressedMetadata_1.Load(Coord);
	}

	return CompressedMetadata;
}
#else
{
	FSSDCompressedSceneInfos CompressedMetadata = CreateCompressedSceneInfos();

	int3 Coord = int3(PixelCoord, /* MipLevel = */ 0);

	if (bPrevFrame)
	{
		CompressedMetadata.VGPR[0] = PrevCompressedMetadata_0.Load(Coord);
	}
	else
	{
		CompressedMetadata.VGPR[0] = CompressedMetadata_0.Load(Coord);
	}
	
	return CompressedMetadata;
}
#endif
