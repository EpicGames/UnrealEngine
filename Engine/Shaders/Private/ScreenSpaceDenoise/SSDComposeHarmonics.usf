// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDDefinitions.ush"
#include "SSDPublicHarmonics.ush"


//------------------------------------------------------- CONFIGS

#define TILE_PIXEL_SIZE 8

#define CONFIG_SIGNAL_PROCESSING SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC

#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
	// Denoise diffuse and specular harmonics at the same time.
	#define MAX_SIGNAL_BATCH_SIZE 2
	#define SIGNAL_ARRAY_SIZE 2
	#define CONFIG_SIGNAL_BATCH_SIZE 2

	// Each harmonic requires input and output RGB.
	#define COMPILE_SIGNAL_COLOR_ARRAY 2

	// Outputing a single composed color for each diffuse and specular.
	#define COMPILE_SIGNAL_COLOR 1

	//#define CONFIG_VIEW_POSITION_BASED_BILATERAL 2
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_INPUT
	#define CONFIG_SIGNAL_RECONSTRUCTED_LAYOUT  SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_RECONSTRUCTION
	#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY
	#define CONFIG_INPUT_TEXTURE_COUNT 4
	#define CONFIG_OUTPUT_TEXTURE_COUNT 2

	#define CONFIG_HARMONIC_COUNT DENOISER_HARMONICS_COUNT

#else
	#error Unknown signal processing.
#endif


//------------------------------------------------------- INCLUDES

#include "SSDSignalFramework.ush"
#include "SSDSignalArray.ush"
#include "SSDSignalBufferEncoding.ush"


//------------------------------------------------------- PARAMETERS

#if !defined(CONFIG_INPUT_TEXTURE_COUNT)
	#error Missing CONFIG_INPUT_TEXTURE_COUNT
#endif

FSSDTexture2D SignalHarmonics_0_Textures_0;
FSSDTexture2D SignalHarmonics_1_Textures_0;
FSSDTexture2D SignalHarmonics_2_Textures_0;
FSSDTexture2D SignalHarmonics_3_Textures_0;

FSSDTexture2D SignalIntegrand_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
FSSDTexture2D SignalHarmonics_0_Textures_1;
FSSDTexture2D SignalHarmonics_1_Textures_1;
FSSDTexture2D SignalHarmonics_2_Textures_1;
FSSDTexture2D SignalHarmonics_3_Textures_1;
FSSDTexture2D SignalIntegrand_Textures_1;
#else
#define SignalHarmonics_0_Textures_1 SignalHarmonics_0_Textures_0
#define SignalHarmonics_1_Textures_1 SignalHarmonics_1_Textures_0
#define SignalHarmonics_2_Textures_1 SignalHarmonics_2_Textures_0
#define SignalHarmonics_3_Textures_1 SignalHarmonics_3_Textures_0
#define SignalIntegrand_Textures_1 SignalIntegrand_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
FSSDTexture2D SignalHarmonics_0_Textures_2;
FSSDTexture2D SignalHarmonics_1_Textures_2;
FSSDTexture2D SignalHarmonics_2_Textures_2;
FSSDTexture2D SignalHarmonics_3_Textures_2;
FSSDTexture2D SignalIntegrand_Textures_2;
#else
#define SignalHarmonics_0_Textures_2 SignalHarmonics_0_Textures_0
#define SignalHarmonics_1_Textures_2 SignalHarmonics_1_Textures_0
#define SignalHarmonics_2_Textures_2 SignalHarmonics_2_Textures_0
#define SignalHarmonics_3_Textures_2 SignalHarmonics_3_Textures_0
#define SignalIntegrand_Textures_2 SignalIntegrand_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
FSSDTexture2D SignalHarmonics_0_Textures_3;
FSSDTexture2D SignalHarmonics_1_Textures_3;
FSSDTexture2D SignalHarmonics_2_Textures_3;
FSSDTexture2D SignalHarmonics_3_Textures_3;
FSSDTexture2D SignalIntegrand_Textures_3;
#else
#define SignalHarmonics_0_Textures_3 SignalHarmonics_0_Textures_0
#define SignalHarmonics_1_Textures_3 SignalHarmonics_1_Textures_0
#define SignalHarmonics_2_Textures_3 SignalHarmonics_2_Textures_0
#define SignalHarmonics_3_Textures_3 SignalHarmonics_3_Textures_0
#define SignalIntegrand_Textures_3 SignalIntegrand_Textures_0
#endif


#if !defined(CONFIG_OUTPUT_TEXTURE_COUNT)
	#error Missing CONFIG_OUTPUT_TEXTURE_COUNT
#endif

FSSDRWTexture2D SignalOutput_UAVs_0;

#if CONFIG_OUTPUT_TEXTURE_COUNT > 1
FSSDRWTexture2D	SignalOutput_UAVs_1;
#else
#define SignalOutput_UAVs_1 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 2
FSSDRWTexture2D	SignalOutput_UAVs_2;
#else
#define SignalOutput_UAVs_2 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 3
FSSDRWTexture2D	SignalOutput_UAVs_3;
#else
#define SignalOutput_UAVs_3 SignalOutput_UAVs_0
#endif


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	// Find out scene buffer UV.
	float2 SceneBufferUV = DispatchThreadId * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;
	if (true)
	{
		SceneBufferUV = clamp(SceneBufferUV, DenoiserBufferBilinearUVMinMax.xy, DenoiserBufferBilinearUVMinMax.zw);
	}
	
	// Read reference meta data.
	float2 ScreenPosition = DenoiserBufferUVToScreenPosition(SceneBufferUV);

	// Sample the harmonics.
	FSSDSignalArray MultiplexedHarmonics[4];
	FSSDSignalFrequencyArray MultiplexedFrequencies;
	{
		SampleMultiplexedSignals(
			SignalHarmonics_0_Textures_0,
			SignalHarmonics_0_Textures_1,
			SignalHarmonics_0_Textures_2,
			SignalHarmonics_0_Textures_3,
			GlobalPointClampedSampler,
			CONFIG_SIGNAL_INPUT_LAYOUT,
			/* MultiplexedSampleId = */ 0,
			/* bNormalizeSample = */ false,
			SceneBufferUV,
			MultiplexedHarmonics[0],
			MultiplexedFrequencies);
	
		SampleMultiplexedSignals(
			SignalHarmonics_1_Textures_0,
			SignalHarmonics_1_Textures_1,
			SignalHarmonics_1_Textures_2,
			SignalHarmonics_1_Textures_3,
			GlobalPointClampedSampler,
			CONFIG_SIGNAL_RECONSTRUCTED_LAYOUT,
			/* MultiplexedSampleId = */ 0,
			/* bNormalizeSample = */ false,
			SceneBufferUV,
			MultiplexedHarmonics[1],
			MultiplexedFrequencies);
	
		SampleMultiplexedSignals(
			SignalHarmonics_2_Textures_0,
			SignalHarmonics_2_Textures_1,
			SignalHarmonics_2_Textures_2,
			SignalHarmonics_2_Textures_3,
			GlobalPointClampedSampler,
			CONFIG_SIGNAL_RECONSTRUCTED_LAYOUT,
			/* MultiplexedSampleId = */ 0,
			/* bNormalizeSample = */ false,
			SceneBufferUV,
			MultiplexedHarmonics[2],
			MultiplexedFrequencies);
	
		SampleMultiplexedSignals(
			SignalHarmonics_3_Textures_0,
			SignalHarmonics_3_Textures_1,
			SignalHarmonics_3_Textures_2,
			SignalHarmonics_3_Textures_3,
			GlobalPointClampedSampler,
			CONFIG_SIGNAL_RECONSTRUCTED_LAYOUT,
			/* MultiplexedSampleId = */ 0,
			/* bNormalizeSample = */ false,
			SceneBufferUV,
			MultiplexedHarmonics[3],
			MultiplexedFrequencies);
	}
	
	// Sample the integrated signal.
	FSSDSignalArray MultiplexedIntegrand;
	SampleMultiplexedSignals(
		SignalIntegrand_Textures_0,
		SignalIntegrand_Textures_1,
		SignalIntegrand_Textures_2,
		SignalIntegrand_Textures_3,
		GlobalPointClampedSampler,
		CONFIG_SIGNAL_RECONSTRUCTED_LAYOUT,
		/* MultiplexedSampleId = */ 0,
		/* bNormalizeSample = */ true,
		SceneBufferUV,
		MultiplexedIntegrand,
		MultiplexedFrequencies);

	// Determine computation that needs to be done.
	FSSDSignalArray OutputSamples = CreateSignalArrayFromScalarValue(0.0);
	UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
	for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
	{
		const uint LowestFrequencyHarmonicId = DENOISER_HARMONICS_COUNT - 1;
		float LowestFrequencyWeight;
		{
			FSSDSignalSample Sample = MultiplexedHarmonics[LowestFrequencyHarmonicId].Array[MultiplexId];
			OutputSamples.Array[MultiplexId] = Sample;
			LowestFrequencyWeight = Sample.SampleCount;
		}

		float3 Color = 0;
		float3 ColorMask = 1.0;

		UNROLL_N(DENOISER_HARMONICS_COUNT)
		for (uint HarmonicId = 0; HarmonicId < DENOISER_HARMONICS_COUNT; HarmonicId++)
		{
			FSSDSignalSample Sample = MultiplexedHarmonics[HarmonicId].Array[MultiplexId];
			FSSDSignalSample NormalizedSample = NormalizeToOneSample(Sample);

			float3 Incoming = NormalizedSample.ColorArray[0].rgb;
			float3 Outgoing = NormalizedSample.ColorArray[1].rgb;
			float3 Stoped = Incoming - Outgoing;

			// TODO(Denoiser).
			float3 HarmonicMask;
			HarmonicMask.r = Incoming.r > 0 ? saturate(Outgoing.r / Incoming.r) : 1.0;
			HarmonicMask.g = Incoming.g > 0 ? saturate(Outgoing.g / Incoming.g) : 1.0;
			HarmonicMask.b = Incoming.b > 0 ? saturate(Outgoing.b / Incoming.b) : 1.0;

			ColorMask *= HarmonicMask;
			
			//if (HarmonicId == View.GeneralPurposeTweak && MultiplexId == 0)
			//{
			//	DebugOutput[DispatchThreadId] = float4(HarmonicMask, 0);
			//}
		}
		
		FSSDSignalSample IntegrandSample = MultiplexedIntegrand.Array[MultiplexId];
		float3 TotalLighting = IntegrandSample.ColorArray[0].rgb;

		float3 FinalLighting = ColorMask * TotalLighting;
		
		//if (MultiplexId == 0)
		//{
		//	if (SceneBufferUV.x > 0.5)
		//		DebugOutput[DispatchThreadId] = float4(TotalLighting, 0);
		//	else
		//		DebugOutput[DispatchThreadId] = float4(TotalLighting, 0);
		//}

		OutputSamples.Array[MultiplexId].SampleCount = LowestFrequencyWeight;
		OutputSamples.Array[MultiplexId].SceneColor.rgb = FinalLighting * LowestFrequencyWeight;
		
		//OutputSamples.Array[MultiplexId].SampleCount = LowestFrequencyWeight;
		//OutputSamples.Array[MultiplexId].SceneColor.rgb = MultiplexedHarmonics[2].Array[MultiplexId].ColorArray[0] * LowestFrequencyWeight;
	}

	// Clamp.
	UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
	for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
	{
		#if COMPILE_SIGNAL_COLOR
			OutputSamples.Array[MultiplexId].SceneColor = max(OutputSamples.Array[MultiplexId].SceneColor, 0);
		#endif
	}
	
	#if CONFIG_VGPR_OPTIMIZATION
		// No need to keep DispatchThreadId, while SceneBufferUV is arround at highest VGPR peak because center of the unique pixel to sample.
		uint2 OutputPixelPostion = BufferUVToBufferPixelCoord(SceneBufferUV);
	#else
		uint2 OutputPixelPostion = ViewportMin + DispatchThreadId;
	#endif 
		
	BRANCH
	if (all(OutputPixelPostion < ViewportMax))
	{
		OutputMultiplexedSignal(
			SignalOutput_UAVs_0,
			SignalOutput_UAVs_1,
			SignalOutput_UAVs_2,
			SignalOutput_UAVs_3,
			CONFIG_SIGNAL_OUTPUT_LAYOUT, CONFIG_SIGNAL_BATCH_SIZE,
			OutputPixelPostion, 
			OutputSamples.Array,
			MultiplexedFrequencies
			);
	}
} // MainCS
