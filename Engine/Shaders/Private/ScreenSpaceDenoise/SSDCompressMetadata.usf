// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDMetadata.ush"
#include "../SceneTextureParameters.ush"


//------------------------------------------------------- CONFIGS

#define TILE_PIXEL_SIZE 8


//------------------------------------------------------- PARAMETERS

uint2 ViewportMin;
uint2 ViewportMax;
float4 ThreadIdToBufferUV;
float4 BufferBilinearUVMinMax;
float2 BufferUVToOutputPixelPosition;

RWTexture2D<uint> CompressedMetadataOutput_0;


//------------------------------------------------------- FUNCTIONS

FSSDSampleSceneInfos FetchCurrentSceneInfosFromGBuffer(float2 ScreenPosition, float2 BufferUV)
{
	float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
	FGBufferData GBufferData = GetGBufferDataFromSceneTextures(BufferUV);
	
	FSSDSampleSceneInfos Infos = CreateSampleSceneInfos();
	Infos.ScreenPosition = ScreenPosition;
	Infos.DeviceZ = DeviceZ;
	Infos.WorldDepth = GBufferData.Depth;
	Infos.WorldNormal = GBufferData.WorldNormal;
	Infos.Roughness = GBufferData.Roughness;
	Infos.ShadingModelID = GBufferData.ShadingModelID;
	
	// Compute translated world position.
	{
		float2 ClipPosition = GetScreenPositionForProjectionType(ScreenPosition, Infos.WorldDepth);
		Infos.TranslatedWorldPosition = mul(float4(ClipPosition, Infos.WorldDepth, 1), View.ScreenToTranslatedWorld).xyz;
	}
	
	// Compute view normal.
	Infos.ViewNormal = mul(float4(Infos.WorldNormal, 0), View.TranslatedWorldToView).xyz;

	return Infos;
}


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	// Find out the UV and screen position.
	float2 SceneBufferUV = DispatchThreadId * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;
	float2 ViewportUV = BufferUVToViewportUV(SceneBufferUV);
	float2 ScreenPosition = ViewportUVToScreenPos(ViewportUV);

	// Fetch current scene metadata.
	FSSDSampleSceneInfos SceneMetadata = FetchCurrentSceneInfosFromGBuffer(ScreenPosition, SceneBufferUV);

	// Compress the metadata.
	FSSDCompressedSceneInfos CompressedMetadata = CompressSampleSceneInfo(DIM_METADATA_LAYOUT, SceneMetadata);

	// No need to keep DispatchThreadId, while SceneBufferUV is arround at highest VGPR peak.
	uint2 OutputPixelPostion = uint2(SceneBufferUV * BufferUVToOutputPixelPosition);

	BRANCH
	if (all(OutputPixelPostion < ViewportMax))
	{
		CompressedMetadataOutput_0[OutputPixelPostion] = CompressedMetadata.VGPR[0];
	}
} // MainCS
