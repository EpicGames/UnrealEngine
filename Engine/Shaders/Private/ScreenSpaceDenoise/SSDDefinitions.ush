// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush"


//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


//------------------------------------------------------- DEBUG CONFIG

/** Turn on/off tile classfication for debugging purpose. */
#define DEBUG_ENABLE_TILE_CLASSIFICATION 1

/** Should output invalid data when know there is a need. */
// TODO(Denoiser): there is an artifact in temporal reprojection.
#define DEBUG_WASTFUL_OUTPUT 1

/** Force the signal to a specific mip level. */
#define DEBUG_MIP_LEVEL -1

/** Debuging mode. */
	#define DEBUG_MODE_DISABLED -1
	#define DEBUG_MODE_BRUTE_FORCE_ONLY 0
	#define DEBUG_MODE_FAST_ONLY 1
#define DEBUG_MODE DEBUG_MODE_DISABLED

/** DIsable any bilateral rejection. */
#ifndef DEBUG_DISABLE_BILATERAL
	#define DEBUG_DISABLE_BILATERAL 0
#endif

/** Prefer to store normalized components in internal texture. */
#define DEBUG_STORE_NORMALIZED_COMPONENTS 1

/** Disable any VGPR compression of the signal. */
#define DEBUG_DISABLE_SIGNAL_COMPRESSION 0

/** Enables debugging output from shaders. */
#define DEBUG_OUTPUT 0


//------------------------------------------------------- SHARED CONSTANTS

// Maximum number of mip level supported.
#define MAX_MIP_LEVEL_COUNT 4

// This constant is to artificially store the signal into higher resolution that theorically should from
// the infered bluring radius by deviding it. 1.0 means nothing happen, but in practice want this >=
// than Nyquist's 2.0 multiplier. Uses 3.0 because this is the size of the 3x3 neighborhood kernel.
#define BLURING_ESTIMATION_MULTIPLIER 3.0

// World bluring radius for a miss.
#define WORLD_RADIUS_INVALID -1
#define WORLD_RADIUS_MISS (100000)

// World bluring radius for a miss.
#define CONFUSION_FACTOR_INVALID -1
#define CONFUSION_FACTOR_MISS 1


// Mip level for sample that are invalid or missed.
#define MIP_LEVEL_INVALID -1
#define MIP_LEVEL_MISS 1024

// Maximum number of buffer multiple signals can be encode to/decoded from.
#define MAX_MULTIPLEXED_TEXTURES 4


//------------------------------------------------------- ENUMS

/** Layouts of the metadata buffer */
	/** Uses standard depth buffer and gbuffer. */
	#define METADATA_BUFFER_LAYOUT_DISABLED 0

	/** Compress depth and world normal. */
	#define METADATA_BUFFER_LAYOUT_DEPTH_NORMAL 1

	/** Compress depth and view normal. */
	#define METADATA_BUFFER_LAYOUT_DEPTH_VIEWNORMAL 2

	/** Compress depth shading model id. */
	#define METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL 3

	/** Uncompressed translated world position and shading model id. */
	#define METADATA_BUFFER_LAYOUT_WORLD_POS_SHADINGMODEL 4




/** Layouts of the signal buffer. */
	/** Buffer layout for the shadow penumbra given as input. */
	#define SIGNAL_BUFFER_LAYOUT_UNINITIALIZED 0xDEAD


	/** Buffer layout for the shadow penumbra given as input. */
	#define SIGNAL_BUFFER_LAYOUT_PENUMBRA_INPUT_NSPP 15

	/** Buffer layout for the shadow penumbra given as input. */
	#define SIGNAL_BUFFER_LAYOUT_PENUMBRA_INJESTION_NSPP 16

	/** Internal buffer layout for the shadow penumbra to be stored in indivual per light histories. */
	#define SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY 11


	/** Buffer layout input for denoising multi polychromatic penumbra. */
	#define SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_INPUT 0x1000

	/** Internal buffer layout for denoising multi polychromatic penumbra. */
	#define SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_RECONSTRUCTION 0x1001

	/** Internal buffer layout for denoising multi polychromatic penumbra. */
	#define SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY 0x1002


	/** Buffer layout for the reflection given by the ray generation shader. */
	#define SIGNAL_BUFFER_LAYOUT_REFLECTIONS_INPUT 3

	/** Buffer layout for the reflection output. */
	#define SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY 4


	/** Buffer layout given by the ray generation shader for ambient occlusion. */
	#define SIGNAL_BUFFER_LAYOUT_AO_INPUT 0xA000

	/** Internal buffer layout for ambient occlusion. */
	#define SIGNAL_BUFFER_LAYOUT_AO_REJECTION 0xA001

	/** Internal buffer layout for ambient occlusion. */
	#define SIGNAL_BUFFER_LAYOUT_AO_HISTORY 0xA002


	/** Buffer layout given by the ray generation shader for diffuse indirect  illumination. */
	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_INPUT_NSPP 0xD100

	/** Internal buffer layout for diffuse indirect illumination. */
	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION 0xD101

	/** Internal buffer layout for diffuse indirect illumination. */
	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY 0xD102


	/** Buffer layout given by the ray generation shader for diffuse indirect  illumination. */
	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC 0xD200


	/** Buffer layout given by SSGI. */
	#define SIGNAL_BUFFER_LAYOUT_SSGI_INPUT 0xD300

	#define SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10 0xD301


	/** Buffer layouts for Lumen diffuse indirect. */
	#define SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_INPUT 0xD400

	#define SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY 0xD401


	/** Buffer layouts for Lumen diffuse indirect. */
	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_INPUT 0xD500

	#define SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_HISTORY 0xD501


/** Defines how the signal is being processed. Matches C++'s ESignalProcessing. */
	/** Shadow penumbra denoising. */
	#define SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK 0

	/** Shadow penumbra denoising. */
	#define SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC 1

	/** Reflection denoising. */
	#define SIGNAL_PROCESSING_REFLECTIONS 2

	/** Ambient occlusion denoising. */
	#define SIGNAL_PROCESSING_AO 3

	/** Diffuse indirect and ambient occlusion. */
	#define SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO 4

	/** Diffuse indirect using spherical harmonic. */
	#define SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC 5

	/** Diffuse indirect using spherical harmonic. */
	#define SIGNAL_PROCESSING_SSGI 6

	/** Diffuse indirect using spherical harmonic. */
	#define SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY 7



/** Defines the color space bitfield. */
	#define COLOR_SPACE_RGB 0x0
	#define COLOR_SPACE_YCOCG 0x1
	#define COLOR_SPACE_LCOCG 0x2

	#define COLOR_SPACE_UNPREMULTIPLY 0x4
	#define COLOR_SPACE_KARIS_WEIGHTING 0x8


/** Sets of sample available for the spatial kernel. */
	// For debug purpose, only sample the center of the kernel.
	#define SAMPLE_SET_1X1 0

	// Filtering
	#define SAMPLE_SET_2X2_BILINEAR 1

	// Stocastic 2x2 that only take one sample.
	#define SAMPLE_SET_2X2_STOCASTIC 13

	// Adapt between bilinear and stocastic 2x2.
	#define SAMPLE_SET_2X2_ADAPTIVE 17

	// Square kernel
	#define SAMPLE_SET_3X3 2
	#define SAMPLE_SET_3X3_SOBEK2018 3
	#define SAMPLE_SET_5X5_WAVELET 4

	#define SAMPLE_SET_3X3_PLUS 5
	#define SAMPLE_SET_3X3_CROSS 6

	#define SAMPLE_SET_NXN 7

	// [ Stackowiak 2015, "Stochastic Screen-Space Reflections" ]
	#define SAMPLE_SET_STACKOWIAK_4_SETS 8

	#define SAMPLE_SET_HEXAWEB 11

	#define SAMPLE_SET_DIRECTIONAL_RECT 14
	#define SAMPLE_SET_DIRECTIONAL_ELLIPSE 15

	#define SAMPLE_SET_RAW_EXPERIMENTAL_KERNEL 16


/** Compression of signal for VGPR occupency. */
	/** The signal is not being compressed. */
	#define SIGNAL_COMPRESSION_DISABLED 0

	/** VGPR compression for spherical harmonic. */
	#define SIGNAL_COMPRESSION_DIFFUSE_INDIRECT_HARMONIC 0xD200


/** Compression of signal for VGPR occupency. */
	/** The signal is not being compressed. */
	#define ACCUMULATOR_COMPRESSION_DISABLED 0

	/** VGPR compression for spherical harmonic. */
	#define ACCUMULATOR_COMPRESSION_PENUMBRA_DRB 0x0001


/** Technic used to compute the world vector betweem neighbor and reference. */
	// Directly use FSSDKernelConfig::RefSceneMetadata.TranslatedWorldPosition. Cost 3 VGPR over entire kernel inner loop.
	#define NEIGHBOR_TO_REF_CACHE_WORLD_POSITION 0

	// Compute by deriving the information of the reference pixel last minute from kernel configuration, trading VALU to save VGPR.
	// Caution: does not work with previous frame reprojection.
	#define NEIGHBOR_TO_REF_LOWEST_VGPR_PRESSURE 1


/** Type to use for texture sampling. */
	#define SIGNAL_TEXTURE_TYPE_FLOAT4 0
	#define SIGNAL_TEXTURE_TYPE_UINT1 1
	#define SIGNAL_TEXTURE_TYPE_UINT2 2
	#define SIGNAL_TEXTURE_TYPE_UINT3 3
	#define SIGNAL_TEXTURE_TYPE_UINT4 4


/** Bilateral settings bitfield. */
	#define BILATERAL_POSITION_BASED(n) (0x0000 + (n) & 0xF)
	#define BILATERAL_NORMAL 0x0010
	#define BILATERAL_TOKOYASHI_LOBE 0x0020
	#define BILATERAL_TOKOYASHI_AXES 0x0040
	#define BILATERAL_TOKOYASHI (BILATERAL_TOKOYASHI_LOBE | BILATERAL_TOKOYASHI_AXES)
	#define BILATERAL_SHADING_MODEL 0x0080

/** Bilateral settings presets for spatial kernels. */
	// No bilateral settings.
	#define BILATERAL_PRESET_DISABLED 0x0000

	#define BILATERAL_PRESET_MONOCHROMATIC_PENUMBRA 0x0011

	#define BILATERAL_PRESET_POLYCHROMATIC_PENUMBRA 0x0022

	#define BILATERAL_PRESET_REFLECTIONS 0x1001
	#define BILATERAL_PRESET_REFLECTIONS_1SPP 0x1002
	#define BILATERAL_PRESET_REFLECTIONS_TAA 0x1003

	#define BILATERAL_PRESET_DIFFUSE 0x2001

	#define BILATERAL_PRESET_SPHERICAL_HARMONIC 0x3001

	#define BILATERAL_PRESET_AO 0x4001
	#define BILATERAL_PRESET_AO_HISTORY 0x4002
	#define BILATERAL_PRESET_PROBE_HIERARCHY 0xF000



//------------------------------------------------------- GLOBAL CONFIGURATION OF ALL DENOISER SHADERS

/** Do normal based anysotropy. */
// TODO(Denoiser): the hierarchical denoiser needs that.
#define CONFIG_NORMAL_ANISOTROPY 0

/** Whether should enable VGPR optimizations in the entry point of the shaders. */
#define CONFIG_VGPR_OPTIMIZATION 1

/** Whether should do VALU optimizations. */
#define CONFIG_VALU_OPTIMIZATIONS 1

/** Whether should compile hint to the shader compiler to force shift some variable to SGPR. */
// TODO(Denoiser): the shader compiler is not doing the right thing.
#define CONFIG_SGPR_HINT_OPTIMIZATION 0

/** Configures whether the world normal should be compressed in FSSDSampleSceneInfos. */
#define CONFIG_COMPRESS_WORLD_NORMAL 1

/** Configures whether the world position should be compressed in FSSDSampleSceneInfos. */
#define CONFIG_COMPRESS_WORLD_POSITION 1

/** By default, the color space stored into intermediary buffer is linear premultiplied RGBA. */
#define STANDARD_BUFFER_COLOR_SPACE COLOR_SPACE_RGB

/** Color space used for reflection history rejection. */
#define REFLECTIONS_REJECTION_COLOR_SPACE (COLOR_SPACE_LCOCG | COLOR_SPACE_UNPREMULTIPLY)

/** Defines the order of the spherical harmonic to denoise. */
// TODO(Denoiser): quality permutation instead?
#define SPHERICAL_HARMONIC_ORDER 2
