// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDDefinitions.ush"


//------------------------------------------------------- CONFIGS

#define TILE_PIXEL_SIZE 8

#define CONFIG_SIGNAL_PROCESSING DIM_SIGNAL_PROCESSING
#define CONFIG_SIGNAL_BATCH_SIZE DIM_SIGNAL_BATCH_SIZE

#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
	#define MAX_SIGNAL_BATCH_SIZE CONFIG_SIGNAL_BATCH_SIZE
	#define SIGNAL_ARRAY_SIZE CONFIG_SIGNAL_BATCH_SIZE

	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_PENUMBRA_INPUT_NSPP
	#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_PENUMBRA_INJESTION_NSPP
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
	#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_UINT2

	#define CONFIG_INPUT_TEXTURE_COUNT DIM_SIGNAL_BATCH_SIZE
	#define CONFIG_OUTPUT_TEXTURE_COUNT ((DIM_SIGNAL_BATCH_SIZE + 1) / 2)

#else
	#error Unknown signal processing.
#endif


//------------------------------------------------------- INCLUDES

#include "SSDSignalFramework.ush"
#include "SSDSignalArray.ush"
#include "SSDSignalBufferEncoding.ush"


//------------------------------------------------------- PARAMETERS

#if !defined(CONFIG_INPUT_TEXTURE_COUNT)
	#error Missing CONFIG_INPUT_TEXTURE_COUNT
#endif

FSSDTexture2D SignalInput_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
FSSDTexture2D SignalInput_Textures_1;
#else
#define SignalInput_Textures_1 SignalInput_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
FSSDTexture2D SignalInput_Textures_2;
#else
#define SignalInput_Textures_2 SignalInput_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
FSSDTexture2D SignalInput_Textures_3;
#else
#define SignalInput_Textures_3 SignalInput_Textures_0
#endif


#if !defined(CONFIG_OUTPUT_TEXTURE_COUNT)
	#error Missing CONFIG_OUTPUT_TEXTURE_COUNT
#endif

FSSDRWTexture2D SignalOutput_UAVs_0;

#if CONFIG_OUTPUT_TEXTURE_COUNT > 1
FSSDRWTexture2D	SignalOutput_UAVs_1;
#else
#define SignalOutput_UAVs_1 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 2
FSSDRWTexture2D	SignalOutput_UAVs_2;
#else
#define SignalOutput_UAVs_2 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 3
FSSDRWTexture2D	SignalOutput_UAVs_3;
#else
#define SignalOutput_UAVs_3 SignalOutput_UAVs_0
#endif


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	// Find out scene buffer UV.
	float2 SceneBufferUV = DispatchThreadId * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;
	if (true)
	{
		SceneBufferUV = clamp(SceneBufferUV, DenoiserBufferBilinearUVMinMax.xy, DenoiserBufferBilinearUVMinMax.zw);
	}
	
	// Read reference meta data.
	FSSDSampleSceneInfos RefSceneMetadata;
	{
		FSSDCompressedSceneInfos CompressedRefSceneMetadata = SampleCompressedSceneMetadata(
			/* bPrevFrame = */ false,
			SceneBufferUV, BufferUVToBufferPixelCoord(SceneBufferUV));
		
		float2 ScreenPosition = DenoiserBufferUVToScreenPosition(SceneBufferUV);
	
		RefSceneMetadata = UncompressSampleSceneInfo(
			CONFIG_METADATA_BUFFER_LAYOUT, /* bPrevFrame = */ false,
			ScreenPosition, CompressedRefSceneMetadata);
	}

	// Sample the reference sample.
	FSSDSignalArray MultiplexedSamples;
	FSSDSignalFrequencyArray MultiplexedFrequencies;
	SampleMultiplexedSignals(
		SignalInput_Textures_0,
		SignalInput_Textures_1,
		SignalInput_Textures_2,
		SignalInput_Textures_3,
		GlobalPointClampedSampler,
		CONFIG_SIGNAL_INPUT_LAYOUT,
		/* MultiplexedSampleId = */ 0,
		/* bNormalizeSample = */ true,
		SceneBufferUV,
		/* out */ MultiplexedSamples,
		/* out */ MultiplexedFrequencies);

	// Performs sample value precomputation.
	UNROLL
	for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
	{
		const uint BatchedSignalId = MultiplexId / CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
		FSSDSignalDomainKnowledge DomainKnowledge = GetSignalDomainKnowledge(BatchedSignalId);

		MultiplexedFrequencies.Array[MultiplexId].WorldBluringRadius = GetSignalWorldBluringRadius(
			MultiplexedFrequencies.Array[MultiplexId], RefSceneMetadata, DomainKnowledge);
	}

	// No need to keep DispatchThreadId, while SceneBufferUV is arround at highest VGPR peak because center of the unique pixel to sample.
	uint2 OutputPixelPostion = BufferUVToBufferPixelCoord(SceneBufferUV);
		
	#if DEBUG_OUTPUT
		//DebugOutput[OutputPixelPostion] = float4(MultiplexedSamples.Array[0].SampleCount, 0, 0, 0);
		DebugOutput[OutputPixelPostion] = float4(MultiplexedFrequencies.Array[0].WorldBluringRadius, 0, 0, 0);
	#endif

	BRANCH
	if (all(OutputPixelPostion < ViewportMax))
	{
		OutputMultiplexedSignal(
			SignalOutput_UAVs_0,
			SignalOutput_UAVs_1,
			SignalOutput_UAVs_2,
			SignalOutput_UAVs_3,
			CONFIG_SIGNAL_OUTPUT_LAYOUT,
			CONFIG_SIGNAL_BATCH_SIZE,
			OutputPixelPostion,
			MultiplexedSamples,
			MultiplexedFrequencies);
	}
} // MainCS
