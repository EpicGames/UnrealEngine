// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDDefinitions.ush"
#include "../Common.ush"
#include "../SceneTextureParameters.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"


//------------------------------------------------------- SHARED CONSTANTS DEPENDING ON CONFIG OF SHADER

#define MAX_COMPRESSED_METADATA_VGPRS 6


//------------------------------------------------------- SHARED FUNCTION

void DecompressDevizeZAndN(uint Compressed, out float DevizeZ, out float3 N)
{
	DevizeZ = f16tof32(Compressed & 0x3FFF);

	uint FaceN = (Compressed >> 15) & 0x7;

	N.x = float((Compressed >> 18) & 0x7F) * rcp(127);
	N.y = float((Compressed >> 25) & 0x7F) * rcp(127);
	N.z = 0.0;

	DecodeNormal(/* inout */ N, FaceN);
}

uint CompressDevizeZAndN(float DevizeZ, float3 N)
{
	uint FaceN;
	EncodeNormal(/* inout */ N, /* out */ FaceN);

	uint2 FaceCood = uint2(clamp(round(127.0 * N.xy), 0, 127.0));
	uint Compressed = f32tof16(DevizeZ) | (FaceN << 15) | (FaceCood.x << 18) | (FaceCood.y << 25);
	return Compressed;
}

/** Return whether this shading model id should be denoised or not. */
bool DenoiseShadingModelID(uint ShadingModelID)
{
	return ShadingModelID != SHADINGMODELID_UNLIT;
}


//------------------------------------------------------- COMMON SCENE INFOS

/** olds commonly used information of the scene of a given sample. */
struct FSSDCompressedSceneInfos
{
	/** Raw compressed buffer, kept fully compressed to have minimal VGPR footprint. */
	uint VGPR[MAX_COMPRESSED_METADATA_VGPRS];
};


FSSDCompressedSceneInfos CreateCompressedSceneInfos()
{
	FSSDCompressedSceneInfos CompressedInfos;
	UNROLL_N(MAX_COMPRESSED_METADATA_VGPRS)
	for (uint i = 0; i < MAX_COMPRESSED_METADATA_VGPRS; i++)
	{
		CompressedInfos.VGPR[i] = 0;
	}
	return CompressedInfos;
}


//------------------------------------------------------- UNCOMPRESSED SCENE INFOS

/** olds commonly used information of the scene of a given sample. */
struct FSSDSampleSceneInfos
{
	/** Raw screen position of the sample. */
	float2 ScreenPosition;

	/** The raw device Z. */
	float DeviceZ;

	/** Raw pixel depth in world space. */
	float WorldDepth;

	/** Roughness of the pixel. */
	float Roughness;
	
	/** Normal of the pixel in world space. */
	float3 WorldNormal;
	
	/** Normal of the pixel in view space. */
	float3 ViewNormal;

	/** Position of the pixel in the translated world frame to save VALU. */
	float3 TranslatedWorldPosition;

	/** Shading model of the scene pixel. */
	uint ShadingModelID;
};

FSSDSampleSceneInfos CreateSampleSceneInfos()
{
	FSSDSampleSceneInfos Infos;
	Infos.WorldDepth              = 0;
	Infos.ScreenPosition          = 0;
	Infos.Roughness               = 0;
	Infos.WorldNormal             = 0;
	Infos.ViewNormal              = 0;
	Infos.TranslatedWorldPosition = 0;
	Infos.ShadingModelID          = SHADINGMODELID_DEFAULT_LIT;
	return Infos;
}


//------------------------------------------------------- UNCOMPRESSED SCENE INFOS

// SUBSTRATE_TODO
ISOLATE
FSSDSampleSceneInfos UncompressSampleSceneInfo(
	const uint CompressedLayout, const bool bIsPrevFrame,
	float2 ScreenPosition,
	FSSDCompressedSceneInfos CompressedInfos)
{
	FSSDSampleSceneInfos Infos = CreateSampleSceneInfos();

	Infos.ScreenPosition = ScreenPosition;

	if (CompressedLayout == METADATA_BUFFER_LAYOUT_DISABLED)
	{
		Infos.WorldDepth = asfloat(CompressedInfos.VGPR[0]);
		Infos.WorldNormal.x = asfloat(CompressedInfos.VGPR[1]);
		Infos.WorldNormal.y = asfloat(CompressedInfos.VGPR[2]);
		Infos.WorldNormal.z = asfloat(CompressedInfos.VGPR[3]);
		Infos.Roughness = asfloat(CompressedInfos.VGPR[4]);
		Infos.ShadingModelID = CompressedInfos.VGPR[5];
		
		Infos.DeviceZ = ConvertToDeviceZ(Infos.WorldDepth);
		Infos.ViewNormal = mul(float4(Infos.WorldNormal, 0), View.TranslatedWorldToView).xyz;
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_DEPTH_NORMAL)
	{
		DecompressDevizeZAndN(CompressedInfos.VGPR[0], /* out */ Infos.DeviceZ, /* out */ Infos.WorldNormal);
		
		Infos.WorldDepth = ConvertFromDeviceZ(Infos.DeviceZ);
		Infos.ViewNormal = mul(float4(Infos.WorldNormal, 0), View.TranslatedWorldToView).xyz;
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_DEPTH_VIEWNORMAL)
	{
		float3 DecodedViewNormal;
		DecompressDevizeZAndN(CompressedInfos.VGPR[0], /* out */ Infos.DeviceZ, /* out */ DecodedViewNormal);
		
		Infos.WorldDepth = ConvertFromDeviceZ(Infos.DeviceZ);

		if (bIsPrevFrame)
		{
			Infos.WorldNormal = mul(float4(DecodedViewNormal, 0), View.PrevViewToTranslatedWorld).xyz;
			
			// TODO(Denoiser): PrevViewToView.
			Infos.ViewNormal = mul(float4(Infos.WorldNormal, 0), View.TranslatedWorldToView).xyz;	
		}
		else
		{
			Infos.ViewNormal = DecodedViewNormal;
			Infos.WorldNormal = mul(float4(Infos.ViewNormal, 0), View.ViewToTranslatedWorld).xyz;
		}
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL)
	{
		Infos.DeviceZ = asfloat(CompressedInfos.VGPR[0]);
		Infos.WorldDepth = ConvertFromDeviceZ(Infos.DeviceZ);
		Infos.ShadingModelID = CompressedInfos.VGPR[1];
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_WORLD_POS_SHADINGMODEL)
	{
		Infos.TranslatedWorldPosition.x = asfloat(CompressedInfos.VGPR[0]);
		Infos.TranslatedWorldPosition.y = asfloat(CompressedInfos.VGPR[1]);
		Infos.TranslatedWorldPosition.z = asfloat(CompressedInfos.VGPR[2]);
		Infos.ShadingModelID = CompressedInfos.VGPR[3];

		if (bIsPrevFrame)
		{
			Infos.WorldDepth = mul(float4(Infos.TranslatedWorldPosition, 1.0), View.TranslatedWorldToView).z;
		}
		else
		{
			Infos.WorldDepth = mul(float4(Infos.TranslatedWorldPosition, 1.0), View.PrevTranslatedWorldToView).z;
		}
		Infos.DeviceZ = ConvertToDeviceZ(Infos.WorldDepth);

		return Infos;
	}
	else
	{
		// ERROR
	}

	// Compute translated world position.
	{
		float4 ClipPosition = float4(GetScreenPositionForProjectionType(ScreenPosition, Infos.WorldDepth), Infos.WorldDepth, 1);

		if (bIsPrevFrame)
		{
			float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
			Infos.TranslatedWorldPosition = mul(ClipPosition, View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;
		}
		else
		{
			Infos.TranslatedWorldPosition = mul(ClipPosition, View.ScreenToTranslatedWorld).xyz;
		}
	}
	return Infos;
}

// SUBSTRATE_TODO
FSSDCompressedSceneInfos CompressSampleSceneInfo(
	const uint CompressedLayout,
	FSSDSampleSceneInfos Infos)
{
	FSSDCompressedSceneInfos CompressedInfos = CreateCompressedSceneInfos();

	if (CompressedLayout == METADATA_BUFFER_LAYOUT_DISABLED)
	{
		CompressedInfos.VGPR[0] = asuint(Infos.WorldDepth);
		CompressedInfos.VGPR[1] = asuint(Infos.WorldNormal.x);
		CompressedInfos.VGPR[2] = asuint(Infos.WorldNormal.y);
		CompressedInfos.VGPR[3] = asuint(Infos.WorldNormal.z);
		CompressedInfos.VGPR[4] = asuint(Infos.Roughness);
		CompressedInfos.VGPR[5] = asuint(Infos.ShadingModelID);
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_DEPTH_NORMAL)
	{
		CompressedInfos.VGPR[0] = CompressDevizeZAndN(Infos.DeviceZ, Infos.WorldNormal);
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_DEPTH_VIEWNORMAL)
	{
		CompressedInfos.VGPR[0] = CompressDevizeZAndN(Infos.DeviceZ, Infos.ViewNormal);
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_FED_DEPTH_SHADINGMODEL)
	{
		CompressedInfos.VGPR[0] = asuint(Infos.DeviceZ);
		CompressedInfos.VGPR[1] = Infos.ShadingModelID;
	}
	else if (CompressedLayout == METADATA_BUFFER_LAYOUT_WORLD_POS_SHADINGMODEL)
	{
		CompressedInfos.VGPR[0] = asuint(Infos.TranslatedWorldPosition.x);
		CompressedInfos.VGPR[1] = asuint(Infos.TranslatedWorldPosition.y);
		CompressedInfos.VGPR[2] = asuint(Infos.TranslatedWorldPosition.z);
		CompressedInfos.VGPR[3] = Infos.ShadingModelID;
	}
	else
	{
		// ERROR
	}

	return CompressedInfos;
}

float3 GetWorldNormal(FSSDSampleSceneInfos SceneMetadata)
{
	return SceneMetadata.WorldNormal;
}

float GetWorldDepth(FSSDSampleSceneInfos SceneMetadata)
{
	return SceneMetadata.WorldDepth;
}

float3 GetTranslatedWorldPosition(FSSDSampleSceneInfos SceneMetadata)
{
	return SceneMetadata.TranslatedWorldPosition;
}


FSSDSampleSceneInfos WaveBroadcastSceneMetadata(const FWaveBroadcastSettings Settings, FSSDSampleSceneInfos SceneMetadata)
{
	FSSDSampleSceneInfos NewSceneMetadata;
	NewSceneMetadata.WorldDepth              = WaveBroadcast(Settings, SceneMetadata.WorldDepth);
	NewSceneMetadata.ScreenPosition          = WaveBroadcast(Settings, SceneMetadata.ScreenPosition);
	NewSceneMetadata.Roughness               = WaveBroadcast(Settings, SceneMetadata.Roughness);
	NewSceneMetadata.WorldNormal             = WaveBroadcast(Settings, SceneMetadata.WorldNormal);
	NewSceneMetadata.ViewNormal              = WaveBroadcast(Settings, SceneMetadata.ViewNormal);
	NewSceneMetadata.TranslatedWorldPosition = WaveBroadcast(Settings, SceneMetadata.TranslatedWorldPosition);
	return NewSceneMetadata;
}
