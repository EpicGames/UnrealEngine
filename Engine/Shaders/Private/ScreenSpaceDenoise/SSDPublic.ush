// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Random.ush"
#include "../MonteCarlo.ush"
#include "../SobolRandom.ush"


/** Bias used on GGX important sample when denoising, to remove part of the tail that creates a lot more noise. 
 *  NOTE: This assumes that GGX sampling is configured to use a polar mapping. */
#define GGX_IMPORTANT_SAMPLE_BIAS 0.1

/** Special value to be encoded in denoiser input when no ray has been shot. */
#define DENOISER_INVALID_HIT_DISTANCE -2.0

/** Special value to be encoded in denoiser input when no ray intersection has been found. */
#define DENOISER_MISS_HIT_DISTANCE -1.0

/** Special value to be encoded in denoiser input when no ray has been shot. */
#define DENOISER_INVALID_CONFUSION_FACTOR -1.0


/** Random generator used for 1 sample per pixel for denoiser input. */
float2 Rand1SPPDenoiserInput(uint2 PixelPos) // TODO(Denoiser): kill
{
	float2 E;

	#if 1
	{
		uint2 Random = Rand3DPCG16( int3( PixelPos, View.StateFrameIndexMod8 ) ).xy;
		E = Rand16ToFloat(Random);
	}
	#elif 0
	{
		uint2 SobolFrame = ComputePixelUniqueSobolRandSample(PixelPos);
		E = SobolIndexToUniformUnitSquare(SobolFrame);
	}
	#else
		#error Miss-configured random generator.
	#endif

	return E;
}

/** Compute the factor to apply to inifinity bluring radius based on the sample's hit distance. */
float ComputeDenoiserConfusionFactor(bool bValid, float DistanceCameraToPixel, float ClosestHitDistance)
{
	return bValid ? (ClosestHitDistance / (DistanceCameraToPixel + ClosestHitDistance)) : DENOISER_INVALID_CONFUSION_FACTOR;
}
