// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "../SHCommon.ush"


/** Compile time settings to encode buffer to the denoiser. */
	#define SSD_ENCODE_CLAMP_COLOR 0x1
	#define SSD_ENCODE_NORMALIZE_MISS 0x2
	#define SSD_ENCODE_NORMALIZE_COLOR 0x4
	#define SSD_ENCODE_NORMALIZE (SSD_ENCODE_NORMALIZE_MISS | SSD_ENCODE_NORMALIZE_COLOR)

/** Compile time settings to decode buffer from the denoiser. */
	#define SSD_DECODE_NORMALIZE 0x1


uint Compress2Floats(float2 v)
{
	// f32tof16() is full rate on GCN.
	return f32tof16(v.x) | (f32tof16(v.y) << 16);
}

float2 Uncompress2Floats(uint v)
{
	// f16tof32() is full rate on GCN.
	return float2(f16tof32(v), f16tof32(v >> 16));
}

uint2 Compress4Floats(float4 v)
{
	return uint2(Compress2Floats(v.xy), Compress2Floats(v.zw));
}

float4 Uncompress4Floats(uint2 v)
{
	return float4(Uncompress2Floats(v.x), Uncompress2Floats(v.y));
}

float4 f32x4tof16(float4 OutRawSample)
{
	return float4(asfloat(Compress2Floats(OutRawSample.xy)), asfloat(Compress2Floats(OutRawSample.zw)), 0, 0);
}


// Encode a SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC formated buffer.
void EncodeDiffuseSphericalHarmonicTexel(
	float SampleCount,
	float MissCount,
	FTwoBandSHVectorRGB ColorSH,
	const uint Options,
	out uint2 OutRawSample[4])
{
	float2 UncompressedSample3;
	UncompressedSample3.x = SampleCount;

	float NormalizationFactor = SampleCount > 0.0 ? rcp(SampleCount) : 0.0;
	if (Options & SSD_ENCODE_NORMALIZE_MISS)
	{
		UncompressedSample3.y = MissCount * NormalizationFactor;
	}
	else
	{
		UncompressedSample3.y = MissCount;
	}

	float ColorNormalizationFactor = 1.0;
	if (Options & SSD_ENCODE_NORMALIZE_COLOR)
	{
		ColorNormalizationFactor = NormalizationFactor;
	}

	float4 UncompressedSample0 = ColorSH.R.V * ColorNormalizationFactor;
	float4 UncompressedSample1 = ColorSH.G.V * ColorNormalizationFactor;
	float4 UncompressedSample2 = ColorSH.B.V * ColorNormalizationFactor;

	if (Options & SSD_ENCODE_CLAMP_COLOR)
	{
		UncompressedSample0 = min(UncompressedSample0, MaxHalfFloat);
		UncompressedSample1 = min(UncompressedSample1, MaxHalfFloat);
		UncompressedSample2 = min(UncompressedSample2, MaxHalfFloat);
	}

	OutRawSample[0] = Compress4Floats(UncompressedSample0);
	OutRawSample[1] = Compress4Floats(UncompressedSample1);
	OutRawSample[2] = Compress4Floats(UncompressedSample2);
	OutRawSample[3] = uint2(Compress2Floats(UncompressedSample3), 0);
}

// Decodes a SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC formated buffer.
void DecodeDiffuseSphericalHarmonicTexel(
	uint2 RawSample[4],
	const uint Options,
	out float OutSampleCount,
	out float OutMissCount,
	out FTwoBandSHVectorRGB OutColorSH)
{
	float2 UncompressedSample3 = Uncompress2Floats(RawSample[3].x);

	float Multiplier;
	if (Options & SSD_DECODE_NORMALIZE)
	{
		Multiplier = 1.0;
		OutSampleCount = UncompressedSample3.x > 0.0 ? 1.0 : 0.0;
	}
	else
	{
		Multiplier = UncompressedSample3.x;
		OutSampleCount = Multiplier;
	}

	OutMissCount = UncompressedSample3.y * Multiplier;
	OutColorSH.R.V = Uncompress4Floats(RawSample[0]) * Multiplier;
	OutColorSH.G.V = Uncompress4Floats(RawSample[1]) * Multiplier;
	OutColorSH.B.V = Uncompress4Floats(RawSample[2]) * Multiplier;
}
