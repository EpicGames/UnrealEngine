// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------- INCLUDE

#include "../Common.ush"
#include "../LightShaderParameters.ush"


//------------------------------------------------------- DEFINES

// Number of armonic being denoised.
#define DENOISER_HARMONICS_COUNT 4
#define DENOISER_HARMONIC_BORDERS_COUNT (DENOISER_HARMONICS_COUNT + 1)


//------------------------------------------------------- FUNCTIONS

float ComputeLightSampleWorldBluringRadius(float3 TranslatedWorldPosition, uint LightType, FLightShaderParameters Light, float HitT)
{
	// Sometime, artists might put occluder very close to the area light compared to they area, that may lead to negative values.
	// TODO(Denoiser): the correct way to fix this is to move this world bluring radius computation into the RGS, and have DistanceFromLight = Ray's MaxT.
	const float ClosestLightDistance = 0.001;

	BRANCH
	if (LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		// TODO(Denoiser): compute this on CPU.
		//return tan(asin(Light.SourceRadius)) * HitT;
		return Light.SourceRadius * rsqrt( 1 - Pow2( Light.SourceRadius ) ) * HitT;
	}
	else if (LightType == LIGHT_TYPE_POINT || LightType == LIGHT_TYPE_SPOT)
	{
		float3 PixelToLightWorldVector = Light.TranslatedWorldPosition - TranslatedWorldPosition;
		float DistanceFromLight = length(PixelToLightWorldVector);

		return Light.SourceRadius * HitT / max(DistanceFromLight - HitT, ClosestLightDistance);
	}
	else // if (LightType == LIGHT_TYPE_RECT)
	{
		float3 PixelToLightWorldVector = Light.TranslatedWorldPosition - TranslatedWorldPosition;
		float DistanceFromLight = length(PixelToLightWorldVector);

		#if CONFIG_VALU_OPTIMIZATIONS
			float Scalar = abs(dot(PixelToLightWorldVector, Light.Direction)) * rsqrt(length2(PixelToLightWorldVector));
		#else
			float Scalar = abs(dot(normalize(PixelToLightWorldVector), Light.Direction));
		#endif

		float2 LightDimensions = GetRectLightDimensions(Light);

		// TODO(Denoiser): area light anysotropy.
		float SmallestLightDimension = min(LightDimensions.x, LightDimensions.y);

		// TODO(Denoiser): HitT on the direction of the ray, but does not DistanceFromLight, witch is bad for large area light.
		return SmallestLightDimension * HitT / max(DistanceFromLight - HitT, ClosestLightDistance);
	}
}

float ProjectWorldDistanceToPixelDistance(float WorldDepth, float WorldDistance)
{
	return WorldDistance * View.ViewSizeAndInvSize.x * 0.5 / GetScreenRayLengthMultiplierForProjectionType(WorldDepth).x;
}

float WorldBluringRadiusToHarmonic(float WorldDepth, float WorldBluringRadius)
{
	float PixelBluringRadius = ProjectWorldDistanceToPixelDistance(WorldDepth, WorldBluringRadius);

	const float TemporalHistoryRejectionDiameter = 3.0;

	return clamp(log2(PixelBluringRadius - (TemporalHistoryRejectionDiameter - 1.0)) * 0.5, 0.0, DENOISER_HARMONICS_COUNT - 1.0);
}


//------------------------------------------------------- Harmonic accumulator for RGS.

// TODO(Denoiser): That is a lot of VGPR for ray gen, implement texture atomic one.
struct FSSDHarmonicAccumulator
{
	float3 HarmonicBorders[DENOISER_HARMONIC_BORDERS_COUNT];
};

FSSDHarmonicAccumulator InitHarmonicAccumulator()
{
	FSSDHarmonicAccumulator Accumulator;
	UNROLL_N(DENOISER_HARMONIC_BORDERS_COUNT)
	for (uint HarmonicBorderId = 0; HarmonicBorderId < DENOISER_HARMONIC_BORDERS_COUNT; HarmonicBorderId++)
	{
		Accumulator.HarmonicBorders[HarmonicBorderId] = 0.0;
	}
	return Accumulator;
}

void AccumulateSampleHarmonic(inout FSSDHarmonicAccumulator Accumulator, float3 Color, float SceneDepth, float HitHarmonic)
{
	UNROLL_N(DENOISER_HARMONIC_BORDERS_COUNT)
	for (uint HarmonicBorderId = 0; HarmonicBorderId < DENOISER_HARMONIC_BORDERS_COUNT; HarmonicBorderId++)
	{
		float Incoming = saturate(1.0 - (float(HarmonicBorderId) - HitHarmonic));
		Accumulator.HarmonicBorders[HarmonicBorderId] += Incoming * Color;
	}
}

void OutputHarmonicAccumulator(
	RWTexture2D<float4> OutputTexture0,
	RWTexture2D<float4> OutputTexture1,
	RWTexture2D<float4> OutputTexture2,
	RWTexture2D<float4> OutputTexture3,
	RWTexture2D<float4> OutputTexture4,
	uint2 PixelPosition,
	FSSDHarmonicAccumulator Accumulator,
	bool bIsValidPixel)
{
	float4 RawOutputs[DENOISER_HARMONIC_BORDERS_COUNT];
	
	UNROLL_N(DENOISER_HARMONIC_BORDERS_COUNT)
	for (uint HarmonicBorderId = 0; HarmonicBorderId < DENOISER_HARMONIC_BORDERS_COUNT; HarmonicBorderId++)
	{
		RawOutputs[HarmonicBorderId] = float4(Accumulator.HarmonicBorders[HarmonicBorderId], bIsValidPixel ? 1.0 : 0.0);
	}
	
	OutputTexture0[PixelPosition] = RawOutputs[0];
	
	#if DENOISER_HARMONIC_BORDERS_COUNT >= 2
		OutputTexture1[PixelPosition] = RawOutputs[1];
	#endif
		
	#if DENOISER_HARMONIC_BORDERS_COUNT >= 3
		OutputTexture2[PixelPosition] = RawOutputs[2];
	#endif

	#if DENOISER_HARMONIC_BORDERS_COUNT >= 4
		OutputTexture3[PixelPosition] = RawOutputs[3];
	#endif

	#if DENOISER_HARMONIC_BORDERS_COUNT >= 5
		OutputTexture4[PixelPosition] = RawOutputs[4];
	#endif
}
