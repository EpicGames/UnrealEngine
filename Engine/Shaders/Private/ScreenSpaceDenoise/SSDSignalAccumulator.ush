// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDSignalCore.ush"
#include "SSDSignalCompression.ush"


#ifndef CONFIG_SIGNAL_VGPR_COMPRESSION
	#define CONFIG_SIGNAL_VGPR_COMPRESSION SIGNAL_COMPRESSION_DISABLED
#endif

#ifndef COMPILE_MOMENT1_ACCUMULATOR
	#define COMPILE_MOMENT1_ACCUMULATOR 0
#endif

#ifndef COMPILE_MOMENT2_ACCUMULATOR
	#define COMPILE_MOMENT2_ACCUMULATOR 0
#endif

#ifndef COMPILE_MINMAX_ACCUMULATOR
	#define COMPILE_MINMAX_ACCUMULATOR 0
#endif

#ifndef COMPILE_MIN_FREQUENCY_ACCUMULATOR
	#define COMPILE_MIN_FREQUENCY_ACCUMULATOR 0
#endif

#ifndef COMPILE_DRB_ACCUMULATOR
	#define COMPILE_DRB_ACCUMULATOR 0
#endif


//------------------------------------------------------- SIGNAL ACCUMULATOR

/** Accumulator of multiple signals. Multiple accumulators are actually implemented in the same structure because lack
 * of templating in shading language, so relly on shader compiler ability's to prune useless stuf.
 */
struct FSSDSignalAccumulator
{
	// Sums of moment.
	#if COMPILE_MOMENT1_ACCUMULATOR
		FSSDSignalSample Moment1;
	#endif

	#if COMPILE_MOMENT2_ACCUMULATOR
		FSSDSignalSample Moment2;
	#endif

	// Compressed form of moment signals.
	#if COMPILE_MOMENT1_ACCUMULATOR
		FSSDCompressedSignalSample CompressedMoment1;
	#endif

	#if COMPILE_MOMENT2_ACCUMULATOR
		FSSDCompressedSignalSample CompressedMoment2;
	#endif

	// Per component min and max values.
	#if COMPILE_MINMAX_ACCUMULATOR
		FSSDSignalSample Min;
		FSSDSignalSample Max;
	#endif

	// Minimal inverse frequency found and intersected when sampling.
	#if COMPILE_MIN_FREQUENCY_ACCUMULATOR
		FSSDSignalFrequency MinFrequency;
	#endif

	// Descending ring bucketing.
	#if COMPILE_DRB_ACCUMULATOR
		FSSDSignalSample Current;
		float CurrentInvFrequency;
		float CurrentTranslucency;

		FSSDSignalSample Previous;
		float PreviousInvFrequency;

		float BorderingRadius;
		float HighestInvFrequency;
	#endif // COMPILE_DRB_ACCUMULATOR
};

/** Informations about cluster of sample */
struct FSSDSampleClusterInfo
{
	// Outter radius boundary of the cluster.
	float OutterBoundaryRadius;
};


FSSDSignalAccumulator CreateSignalAccumulator()
{
	FSSDSignalAccumulator Accumulator;
	
	#if COMPILE_MOMENT1_ACCUMULATOR
		Accumulator.Moment1 = CreateSignalSampleFromScalarValue(0.0);
	#endif

	#if COMPILE_MOMENT2_ACCUMULATOR
		Accumulator.Moment2 = CreateSignalSampleFromScalarValue(0.0);
	#endif	

	#if COMPILE_MOMENT1_ACCUMULATOR
		Accumulator.CompressedMoment1 = CreateCompressedSignalSampleFromScalarValue(0.0, CONFIG_SIGNAL_VGPR_COMPRESSION);
	#endif

	#if COMPILE_MOMENT2_ACCUMULATOR
		Accumulator.CompressedMoment2 = CreateCompressedSignalSampleFromScalarValue(0.0, CONFIG_SIGNAL_VGPR_COMPRESSION);
	#endif
	
	#if COMPILE_MINMAX_ACCUMULATOR
		Accumulator.Min = CreateSignalSampleFromScalarValue(INFINITE_FLOAT);
		Accumulator.Max = CreateSignalSampleFromScalarValue(0.0); // TODO(Denoiser): -INFINITE_FLOAT? otherwise there is a risk to color clamp with YCoCg.
	#endif
		
	#if COMPILE_MIN_FREQUENCY_ACCUMULATOR
		Accumulator.MinFrequency.ClosestHitDistance = INFINITE_FLOAT;
		Accumulator.MinFrequency.WorldBluringRadius = WORLD_RADIUS_MISS;
		Accumulator.MinFrequency.ConfusionFactor = CONFUSION_FACTOR_MISS;
	#endif

	// DRB initialization.
	#if COMPILE_DRB_ACCUMULATOR
	{
		Accumulator.Current = CreateSignalSampleFromScalarValue(0.0);
		Accumulator.CurrentInvFrequency = 0.0;
		Accumulator.CurrentTranslucency = 0.0;
		
		Accumulator.Previous = CreateSignalSampleFromScalarValue(0.0);
		Accumulator.PreviousInvFrequency = 0.0;

		Accumulator.BorderingRadius = 0.0;
		Accumulator.HighestInvFrequency = 0.0;
	}
	#endif

	return Accumulator;
}


//------------------------------------------------------- SIGNAL INFORMATION

struct FSSDSampleAccumulationInfos
{
	// Sample to accumulate.
	FSSDSignalSample Sample;

	// Frequency of the sample.
	FSSDSignalFrequency Frequency;

	// Weight of the sample in the kernel.
	float FinalWeight;

	// 1 / SignalFrequency
	float InvFrequency;
};


// Transform the world bluring distance due to pixel size with some conservative to be forgiving
float AmendWorldBluringRadiusCausedByPixelSize(float WorldBluringDistance)
{
	float Multiplier = 1;

	// The distance between two pixel is 2 times the radius of the pixel.
	Multiplier *= 2;

	// Need to take into account the furthearest pixel of 3x3 neighborhood.
	Multiplier *= sqrt(2);

	return WorldBluringDistance * Multiplier;
}



//------------------------------------------------------- ONE BIN AVG & SQUARE AVG ACCUMULATION

/** Accumulate a sample withing the accumulator. */
void AccumulateSampleInOneBin(inout FSSDSignalAccumulator Accumulator, FSSDSampleAccumulationInfos SampleInfos)
{
	float SampleWeight = SampleInfos.FinalWeight;
	
	#if COMPILE_MOMENT1_ACCUMULATOR
		Accumulator.Moment1 = AddSignal(Accumulator.Moment1, MulSignal(SampleInfos.Sample, SampleWeight));
	#endif
	
	#if COMPILE_MOMENT2_ACCUMULATOR
		Accumulator.Moment2 = AddSignal(Accumulator.Moment2, MulSignal(PowerSignal(SampleInfos.Sample, 2), SampleWeight));
	#endif
}


//------------------------------------------------------- MIN MAX tracking

/** Tracks min and max values accumulator. */
void AccumulateSampleInDomainBoundaries(inout FSSDSignalAccumulator Accumulator, FSSDSampleAccumulationInfos SampleInfos)
{
	FLATTEN
	if (SampleInfos.FinalWeight > 0)
	{
		#if COMPILE_MINMAX_ACCUMULATOR
		{
			Accumulator.Min = MinSignal(Accumulator.Min, SampleInfos.Sample);
			Accumulator.Max = MaxSignal(Accumulator.Max, SampleInfos.Sample);
		}
		#endif
		
		// Track the minimal frequency inverse.
		
		#if COMPILE_MIN_FREQUENCY_ACCUMULATOR
		if (SampleInfos.InvFrequency != WORLD_RADIUS_INVALID)
		{
			Accumulator.MinFrequency = MinSignalFrequency(Accumulator.MinFrequency, SampleInfos.Frequency);
		}
		#endif
	}
}


//------------------------------------------------------- DESCENDING RING BUCKETING
// [ SIGGRAPH 2018, "A life of bokeh" ]

#if COMPILE_DRB_ACCUMULATOR

void StartAccumulatingClusterInDRB(
	FSSDSampleSceneInfos RefSceneMetadata,
	inout FSSDSignalAccumulator Accumulator,
	FSSDSampleClusterInfo ClusterInfo)
{
	const float ErrorCorrection = 1;
	
	// Compute the bluring radius of the output pixel itself.
	float RefPixelWorldBluringRadius = AmendWorldBluringRadiusCausedByPixelSize(ComputeWorldBluringRadiusCausedByPixelSize(RefSceneMetadata));

	Accumulator.BorderingRadius = RefPixelWorldBluringRadius * (ClusterInfo.OutterBoundaryRadius + ErrorCorrection);

	// TODO(Denoiser): could this be constant.
	Accumulator.HighestInvFrequency = Accumulator.BorderingRadius * 2;

	// Reset current bucket.
	{
		Accumulator.Current = CreateSignalSampleFromScalarValue(0.0);
		Accumulator.CurrentInvFrequency = 0.0;
		Accumulator.CurrentTranslucency = 0.0;
	}

	#if CONFIG_SGPR_HINT_OPTIMIZATION && 0
	{
		Accumulator.BorderingRadius = ToScalarMemory(Accumulator.BorderingRadius);
		Accumulator.HighestInvFrequency = ToScalarMemory(Accumulator.HighestInvFrequency);
	}
	#endif
}

void AccumulateSampleInDRB(inout FSSDSignalAccumulator Accumulator, FSSDSampleAccumulationInfos A)
{
	float ClampedInvFrequency = min(A.InvFrequency, Accumulator.HighestInvFrequency);

	// Compare the sample's frequency with the bucket bordering radius.
	float bBelongsToPrevious = saturate(ClampedInvFrequency - Accumulator.BorderingRadius + 0.5);
	float bBelongsToCurrent = 1.0 - bBelongsToPrevious;
	
	// Accumulate current bucket.
	{
		float CurrentWeight = A.FinalWeight * bBelongsToCurrent;
		Accumulator.Current = AddSignal(Accumulator.Current, MulSignal(A.Sample, CurrentWeight));
		Accumulator.CurrentInvFrequency += A.Sample.SampleCount * ClampedInvFrequency * CurrentWeight;
	}

	// Accumulate current bucket translucency.
	{
		float SampleTranslucency = saturate(ClampedInvFrequency - Accumulator.BorderingRadius);
		
		// TODO(Denoiser): do need (A.Sample.MissCount * SafeRcp(A.Sample.SampleCount)) computation onen 1spp reconstruction.
		// TODO(Denoiser): could pass down a normalised version of A.Sample to avoid Rcp.
		float R = A.Sample.MissCount * SafeRcp(A.Sample.SampleCount);
		
		float Translucency = lerp(R, 1, SampleTranslucency);
		float TranslucencyWeight = 1;

		Accumulator.CurrentTranslucency += Translucency * TranslucencyWeight;
	}

	// Accumulate previous bucket.
	{
		float PreviousWeight = A.FinalWeight * bBelongsToPrevious;
		Accumulator.Previous = AddSignal(Accumulator.Previous, MulSignal(A.Sample, PreviousWeight));
		Accumulator.PreviousInvFrequency += A.Sample.SampleCount * ClampedInvFrequency * PreviousWeight;
	}
}

void DijestSampleClusterInDRB(
	inout FSSDSignalAccumulator Accumulator,
	uint RingId, uint SampleCount)
{
	//if (RingId != 0)
	//	return;

	if (Accumulator.Current.SampleCount == 0.0)
	{
		return;
	}
	
	// Opacity of the current bucket.
	float CurrentClusterOpacity = saturate(1 - Accumulator.CurrentTranslucency * rcp(float(SampleCount)));
	
	// Compute current and previous Coc radii.
	float PreviousInvFrequency = Accumulator.PreviousInvFrequency * SafeRcp(Accumulator.Previous.SampleCount);
	float CurrentInvFrequency = Accumulator.CurrentInvFrequency * rcp(Accumulator.Current.SampleCount);

	// Whether current bucket is occluding previous bucket.
	// TODO(Denoiser): put a contrast.
	float bOccludingCluster = saturate((PreviousInvFrequency - CurrentInvFrequency) * rcp(Accumulator.BorderingRadius));

	// Compute final factor to use to with previous bucket.
	float PreviousBucketFactor = (Accumulator.Previous.SampleCount == 0.0) ? 0.0 : (1.0 - CurrentClusterOpacity * bOccludingCluster);

	// Normalize the previous bucket to respect the desired frequency bucket opacity.
	// PreviousBucketFactor *= Accumulator.Current.SampleCount * SafeRcp(Accumulator.Previous.SampleCount);

	Accumulator.Previous = AddSignal(MulSignal(Accumulator.Previous, PreviousBucketFactor), Accumulator.Current);
	Accumulator.PreviousInvFrequency = Accumulator.PreviousInvFrequency * PreviousBucketFactor + Accumulator.CurrentInvFrequency;
}

#endif // COMPILE_DRB_ACCUMULATOR


//------------------------------------------------------- COMPRESSION & DECOMPRESSION

void CompressSignalAccumulator(inout FSSDSignalAccumulator Accumulator)
{
	#if COMPILE_MOMENT1_ACCUMULATOR
		CompressSignalSample(Accumulator.Moment1, CONFIG_SIGNAL_VGPR_COMPRESSION, /* out */ Accumulator.CompressedMoment1);
	#endif
	
	#if COMPILE_MOMENT2_ACCUMULATOR
		CompressSignalSample(Accumulator.Moment2, CONFIG_SIGNAL_VGPR_COMPRESSION, /* out */ Accumulator.CompressedMoment2);
	#endif
}

void UncompressSignalAccumulator(inout FSSDSignalAccumulator Accumulator)
{
	#if COMPILE_MOMENT1_ACCUMULATOR
		UncompressSignalSample(Accumulator.CompressedMoment1, CONFIG_SIGNAL_VGPR_COMPRESSION, /* inout */ Accumulator.Moment1);
	#endif
	
	#if COMPILE_MOMENT2_ACCUMULATOR
		UncompressSignalSample(Accumulator.CompressedMoment2, CONFIG_SIGNAL_VGPR_COMPRESSION, /* inout */ Accumulator.Moment2);
	#endif
}


//------------------------------------------------------- SPATIAL KERNEL ENTRY POINT.

/** Start accumulating a cluster of samples. */
void StartAccumulatingCluster(
	FSSDSampleSceneInfos RefSceneMetadata,
	inout FSSDSignalAccumulator Accumulator,
	FSSDSampleClusterInfo ClusterInfo)
{
	#if COMPILE_DRB_ACCUMULATOR
		StartAccumulatingClusterInDRB(RefSceneMetadata, /* inout */ Accumulator, ClusterInfo);
	#endif
}

/** Accumulate a sample withing the accumulator. */
void AccumulateSample(
	inout FSSDSignalAccumulator Accumulator,
	FSSDSampleAccumulationInfos SampleInfos)
{
	UncompressSignalAccumulator(/* inout */ Accumulator);

	AccumulateSampleInOneBin(/* inout */ Accumulator, SampleInfos);
	AccumulateSampleInDomainBoundaries(/* inout */ Accumulator, SampleInfos);

	#if COMPILE_DRB_ACCUMULATOR
		AccumulateSampleInDRB(/* inout */ Accumulator, SampleInfos);
	#endif // COMPILE_DRB_ACCUMULATOR

	CompressSignalAccumulator(/* inout */ Accumulator);
}

/** Digest the accumulated cluster of samples. */
void DijestAccumulatedClusterSamples(
	inout FSSDSignalAccumulator Accumulator,
	uint RingId, uint SampleCount)
{
	#if COMPILE_DRB_ACCUMULATOR
		DijestSampleClusterInDRB(/* inout */ Accumulator, RingId, SampleCount);
	#endif
}
