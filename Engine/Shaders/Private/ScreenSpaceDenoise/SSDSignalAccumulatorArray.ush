// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDSignalAccumulator.ush"


// Maximum number of VGPR that can be used to compress accumulator array. Currently bound by ACCUMULATOR_COMPRESSION_PENUMBRA_DRB.
#define MAX_ACCUMULATOR_COMPRESSED_VGPRS 20


/** Structure that is just an array of accumulators, to work arround shader compiler bug on "out" of arrays. */
struct FSSDSignalAccumulatorArray
{
	FSSDSignalAccumulator Array[SIGNAL_ARRAY_SIZE];
};

/** Accumulator array that is compressed to have lower VGPR footprint. */
struct FSSDCompressedSignalAccumulatorArray
{
	uint VGPR[MAX_ACCUMULATOR_COMPRESSED_VGPRS];
};

/** Created an initialised accumulator array. */
FSSDSignalAccumulatorArray CreateSignalAccumulatorArray()
{
	FSSDSignalAccumulatorArray Accumulators;
	
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint i = 0; i < SIGNAL_ARRAY_SIZE; i++)
	{
		Accumulators.Array[i] = CreateSignalAccumulator();
	}
	return Accumulators;
}

/** Created an uninitialized accumulator array that is compressed. */
FSSDCompressedSignalAccumulatorArray CreateUninitialisedCompressedAccumulatorArray()
{
	FSSDCompressedSignalAccumulatorArray CompressedAccumulators;
	UNROLL_N(MAX_ACCUMULATOR_COMPRESSED_VGPRS)
	for (uint i = 0; i < MAX_ACCUMULATOR_COMPRESSED_VGPRS; i++)
	{
		CompressedAccumulators.VGPR[i] = 0;
	}
	return CompressedAccumulators;
}

/** Compress the accumulator array. */
FSSDCompressedSignalAccumulatorArray CompressAccumulatorArray(in FSSDSignalAccumulatorArray Accumulators, uint CompressionLayout)
{
	// Initialize the float2 arrays to be compressed at half in VGPRs.
	float2 RawArray[MAX_ACCUMULATOR_COMPRESSED_VGPRS];
	{
		UNROLL_N(MAX_ACCUMULATOR_COMPRESSED_VGPRS)
		for (uint i = 0; i < 2 * MAX_ACCUMULATOR_COMPRESSED_VGPRS; i++)
		{
			RawArray[i].x = 0;
			RawArray[i].y = 0;
		}
	}
	
	if (CompressionLayout == ACCUMULATOR_COMPRESSION_DISABLED)
	{
		// NOP
	}
	#if COMPILE_DRB_ACCUMULATOR && COMPILE_MIN_FREQUENCY_ACCUMULATOR && 0
	else if (CompressionLayout == ACCUMULATOR_COMPRESSION_PENUMBRA_DRB)
	{
		#error Broken
		UNROLL_N(MAX_SIGNAL_BATCH_SIZE)
		for (uint SignalBatchId = 0; SignalBatchId < MAX_SIGNAL_BATCH_SIZE; SignalBatchId++)
		{
			const FSSDSignalAccumulator Accum = Accumulators.Array[SignalBatchId];

			RawArray[SignalBatchId * 5 + 0].x = Accum.Current.SampleCount;
			RawArray[SignalBatchId * 5 + 0].y = Accum.Current.MissCount;
			RawArray[SignalBatchId * 5 + 1].x = Accum.CurrentInvFrequency;
			RawArray[SignalBatchId * 5 + 1].y = Accum.CurrentTranslucency;
			RawArray[SignalBatchId * 5 + 2].x = Accum.Previous.SampleCount;
			RawArray[SignalBatchId * 5 + 2].y = Accum.Previous.MissCount;
			RawArray[SignalBatchId * 5 + 3].x = Accum.PreviousInvFrequency;
			RawArray[SignalBatchId * 5 + 3].y = Accum.MinFrequency.WorldBluringRadius;
			RawArray[SignalBatchId * 5 + 4].x = Accum.HighestInvFrequency;
			RawArray[SignalBatchId * 5 + 4].y = Accum.BorderingRadius;
		} // for (uint SignalMultiplexId = 0; SignalMultiplexId < MAX_SIGNAL_BATCH_SIZE; SignalMultiplexId++)
	}
	#endif // COMPILE_DRB_ACCUMULATOR && COMPILE_MIN_FREQUENCY_ACCUMULATOR

	// Compress.
	FSSDCompressedSignalAccumulatorArray CompressedAccumulators;
	UNROLL_N(MAX_ACCUMULATOR_COMPRESSED_VGPRS)
	for (uint i = 0; i < MAX_ACCUMULATOR_COMPRESSED_VGPRS; i++)
	{
		CompressedAccumulators.VGPR[i] = Compress2Floats(RawArray[i]);
	}

	return CompressedAccumulators;
} // CompressAccumulatorArray()

/** Uncompress the accumulator array. */
FSSDSignalAccumulatorArray UncompressAccumulatorArray(in FSSDCompressedSignalAccumulatorArray CompressedAccumulators, uint CompressionLayout)
{
	// Uncompress VGPRs to floats.
	float2 RawArray[MAX_ACCUMULATOR_COMPRESSED_VGPRS];
	{
		UNROLL_N(MAX_ACCUMULATOR_COMPRESSED_VGPRS)
		for (uint i = 0; i < MAX_ACCUMULATOR_COMPRESSED_VGPRS; i++)
		{
			RawArray[i] = Uncompress2Floats(CompressedAccumulators.VGPR[i]);
		}
	}

	FSSDSignalAccumulatorArray Accumulators = CreateSignalAccumulatorArray();

	if (CompressionLayout == ACCUMULATOR_COMPRESSION_DISABLED)
	{
		// NOP to not clobber information in OutSample that remained uncompressed.
	}
	#if COMPILE_DRB_ACCUMULATOR && COMPILE_MIN_FREQUENCY_ACCUMULATOR && 0
	else if (CompressionLayout == ACCUMULATOR_COMPRESSION_PENUMBRA_DRB)
	{
		// TODO(Denoiser): CONFIG_SIGNAL_BATCH_SIZE?
		UNROLL_N(MAX_SIGNAL_BATCH_SIZE)
		for (uint SignalBatchId = 0; SignalBatchId < MAX_SIGNAL_BATCH_SIZE; SignalBatchId++)
		{
			FSSDSignalAccumulator Accum = CreateSignalAccumulator();

			Accum.Current.SampleCount         = RawArray[SignalBatchId * 5 + 0].x;
			Accum.Current.MissCount           = RawArray[SignalBatchId * 5 + 0].y;
			Accum.CurrentInvFrequency         = RawArray[SignalBatchId * 5 + 1].x;
			Accum.CurrentTranslucency         = RawArray[SignalBatchId * 5 + 1].y;
			Accum.Previous.SampleCount        = RawArray[SignalBatchId * 5 + 2].x;
			Accum.Previous.MissCount          = RawArray[SignalBatchId * 5 + 2].y;
			Accum.PreviousInvFrequency        = RawArray[SignalBatchId * 5 + 3].x;
			Accum.MinInvFrequency             = RawArray[SignalBatchId * 5 + 3].y;
			Accum.HighestInvFrequency         = RawArray[SignalBatchId * 5 + 4].x;
			Accum.BorderingRadius             = RawArray[SignalBatchId * 5 + 4].y;

			Accumulators.Array[SignalBatchId] = Accum;
		} // for (uint SignalBatchId = 0; SignalBatchId < MAX_SIGNAL_BATCH_SIZE; SignalBatchId++)
	}
	#endif // COMPILE_DRB_ACCUMULATOR && COMPILE_MIN_FREQUENCY_ACCUMULATOR

	return Accumulators;
} // UncompressAccumulatorArray()
