// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDSignalCore.ush"


FSSDSignalArray MulSignalArray(FSSDSignalArray Samples, float Scalar[SIGNAL_ARRAY_SIZE])
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = MulSignal(Samples.Array[BatchedSignalId], Scalar[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray AddSignalArray(FSSDSignalArray SamplesA, FSSDSignalArray SamplesB)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = AddSignal(SamplesA.Array[BatchedSignalId], SamplesB.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray MinusSignalArray(FSSDSignalArray Samples)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = MinusSignal(Samples.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray SubtractSignalArray(FSSDSignalArray SamplesA, FSSDSignalArray SamplesB)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = SubtractSignal(SamplesA.Array[BatchedSignalId], SamplesB.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray AbsSignalArray(FSSDSignalArray Samples)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = AbsSignal(Samples.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray SqrtSignalArray(FSSDSignalArray Samples)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = SqrtSignal(Samples.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray PowerSignalArray(FSSDSignalArray Samples, float Exponent)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = PowerSignal(Samples.Array[BatchedSignalId], Exponent);
	}
	return OutSamples;
}

FSSDSignalArray MinSignalArray(FSSDSignalArray SamplesA, FSSDSignalArray SamplesB)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = MinSignal(SamplesA.Array[BatchedSignalId], SamplesB.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray MaxSignalArray(FSSDSignalArray SamplesA, FSSDSignalArray SamplesB)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = MaxSignal(SamplesA.Array[BatchedSignalId], SamplesB.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray ClampSignalArray(FSSDSignalArray Samples, FSSDSignalArray SamplesMin, FSSDSignalArray SamplesMax)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = ClampSignal(Samples.Array[BatchedSignalId], SamplesMin.Array[BatchedSignalId], SamplesMax.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray LerpSignalArray(FSSDSignalArray Samples0, FSSDSignalArray Samples1, float Interp[SIGNAL_ARRAY_SIZE])
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = LerpSignal(Samples0.Array[BatchedSignalId], Samples1.Array[BatchedSignalId], Interp[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray NormalizeToOneSampleArray(FSSDSignalArray Samples)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = NormalizeToOneSample(Samples.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalArray WaveBroadcastSignalArray(const FWaveBroadcastSettings BroadcastSettings, FSSDSignalArray Samples)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = WaveBroadcastSignal(BroadcastSettings, Samples.Array[BatchedSignalId]);
	}
	return OutSamples;
}

FSSDSignalFrequencyArray WaveBroadcastSignalFrequenciesArray(const FWaveBroadcastSettings BroadcastSettings, FSSDSignalFrequencyArray Frequencies)
{
	FSSDSignalFrequencyArray OutFrequencies;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutFrequencies.Array[BatchedSignalId] = WaveBroadcastSignalFrequency(BroadcastSettings, Frequencies.Array[BatchedSignalId]);
	}
	return OutFrequencies;
}
