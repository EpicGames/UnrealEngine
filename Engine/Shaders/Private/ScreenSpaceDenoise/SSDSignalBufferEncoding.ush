// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDSignalCore.ush"
#include "SSDPublicBufferEncoding.ush"
#include "../Lumen/LumenBufferEncoding.ush"

/** Whether the color should be clamped when encoding signal. */
#define CONFIG_ENCODING_CLAMP_COLOR 1

/** Selects the type that should be used when sampling a buffer */
#ifndef CONFIG_SIGNAL_INPUT_TEXTURE_TYPE
	#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
#endif

/** Selects the type that should be used when sampling a buffer */
#ifndef CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE
	#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
#endif


#if CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
	#define FSSDRawSample float4
	#define FSSDTexture2D Texture2D
#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT1 || CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT2 || CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT3 || CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT4
	#if CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT1
		#define FSSDRawSample uint
	#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT2
		#define FSSDRawSample uint2
	#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT3
		#define FSSDRawSample uint3
	#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT4
		#define FSSDRawSample uint4
	#else
		#error Unknown input type for a signal texture.
	#endif
	#define FSSDTexture2D Texture2D<FSSDRawSample>
#else
	#error Unknown input type for a signal texture.
#endif

#if CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
	#define FSSDOutputRawSample float4
#elif CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT1
	#define FSSDOutputRawSample uint
#elif CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT2
	#define FSSDOutputRawSample uint2
#elif CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT3
	#define FSSDOutputRawSample uint3
#elif CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT4
	#define FSSDOutputRawSample uint4
#else
	#error Unknown output type for a signal texture.
#endif

#define FSSDRWTexture2D RWTexture2D<FSSDOutputRawSample>


float3 ClampColorForEncoding(float3 Color)
{
	#if CONFIG_ENCODING_CLAMP_COLOR
		Color = min(Color, MaxHalfFloat);
	#endif
	return Color;
}

float4 ClampColorForEncoding(float4 Color)
{
	return float4(ClampColorForEncoding(Color.rgb), Color.a);
}


/** Raw data layout when sampling input texture of the denoiser. */
struct FSSDCompressedMultiplexedSample
{
	FSSDRawSample VGPRArray[MAX_MULTIPLEXED_TEXTURES];
};


/** Decode input signal sample from raw float. */
void DecodeMultiplexedSignalsFromFloat4(
	const uint SignalBufferLayout,
	const uint MultiplexedSampleId,
	const bool bNormalizeSample,
	float4 RawSample[MAX_MULTIPLEXED_TEXTURES],
	out FSSDSignalArray OutSamples,
	out FSSDSignalFrequencyArray OutSampleFrequencies)
{
	OutSamples = CreateSignalArrayFromScalarValue(0.0);
	OutSampleFrequencies = CreateInvalidSignalFrequencyArray();

	if (0)
	{
	
	}
	#if 1
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_UNINITIALIZED)
	{
		// trash out the output sample to identify quickly a miss configured buffer layout.
		OutSamples = CreateSignalArrayFromScalarValue(-INFINITE_FLOAT);
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_PENUMBRA_INPUT_NSPP)
	{
		UNROLL_N(MAX_SIGNAL_BATCH_SIZE)
		for (uint BatchSignalId = 0; BatchSignalId < MAX_SIGNAL_BATCH_SIZE; BatchSignalId++)
		{
			uint MultiplexId = BatchSignalId;
			// TODO(Denoiser): feed the actual number of sample.
			OutSamples.Array[MultiplexId].SampleCount = (RawSample[MultiplexId].g == DENOISER_INVALID_HIT_DISTANCE ? 0.0 : 1.0);
			OutSamples.Array[MultiplexId].MissCount = OutSamples.Array[MultiplexId].SampleCount * RawSample[MultiplexId].r;
			OutSamples.Array[MultiplexId].TransmissionDistance = OutSamples.Array[MultiplexId].SampleCount * RawSample[MultiplexId].a;

			OutSampleFrequencies.Array[MultiplexId].ClosestHitDistance = RawSample[MultiplexId].g;

			FLATTEN
			if (OutSamples.Array[MultiplexId].SampleCount == 0)
			{
				OutSampleFrequencies.Array[MultiplexId].ClosestHitDistance = DENOISER_INVALID_HIT_DISTANCE;
			}
			else if (OutSamples.Array[MultiplexId].MissCount > 0.999)
			{
				OutSampleFrequencies.Array[MultiplexId].ClosestHitDistance = DENOISER_MISS_HIT_DISTANCE;
			}
		}
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY)
	{
		UNROLL_N(MAX_SIGNAL_BATCH_SIZE)
		for (uint MultiplexId = 0; MultiplexId < MAX_SIGNAL_BATCH_SIZE; MultiplexId++)
		{
			float4 Channels = RawSample[MultiplexId].xyzw;
			
			float SampleCount = bNormalizeSample ? (Channels.g > 0 ? 1 : 0) : (Channels.g);

			OutSamples.Array[MultiplexId].MissCount = Channels.r * SampleCount;
			OutSamples.Array[MultiplexId].SampleCount = SampleCount;
			OutSamples.Array[MultiplexId].TransmissionDistance = Channels.a * SampleCount;

			OutSampleFrequencies.Array[MultiplexId].WorldBluringRadius = Channels.b;
		}
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY >= 2 && SIGNAL_ARRAY_SIZE >= 2
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_INPUT)
	{
		// The RGS output same weigth for diffuse and specular given it depends only on shading model id.
		float Weight = RawSample[0].a;

		if (bNormalizeSample)
		{
			Weight = Weight > 0.0 ? 1.0 : 0.0;
		}

		// Decode diffuse
		{
			const uint MultiplexId = 0;

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].ColorArray[0] = RawSample[0].rgb * Weight;
			OutSamples.Array[MultiplexId].ColorArray[1] = RawSample[1].rgb * Weight;
		}
		
		// Decode specular
		{
			const uint MultiplexId = 1;

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].ColorArray[0] = RawSample[2].rgb * Weight;
			OutSamples.Array[MultiplexId].ColorArray[1] = RawSample[3].rgb * Weight;
		}
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_RECONSTRUCTION)
	{
		// Decode diffuse
		{
			const uint MultiplexId = 0;

			float Weight = RawSample[0].a;
			if (bNormalizeSample)
			{
				Weight = Weight > 0.0 ? 1.0 : 0.0;
			}

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].ColorArray[0] = RawSample[0].rgb * Weight;
			OutSamples.Array[MultiplexId].ColorArray[1] = RawSample[1].rgb * Weight;
		}
		
		// Decode specular
		{
			const uint MultiplexId = 1;

			float Weight = RawSample[2].a;
			if (bNormalizeSample)
			{
				Weight = Weight > 0.0 ? 1.0 : 0.0;
			}

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].ColorArray[0] = RawSample[2].rgb * Weight;
			OutSamples.Array[MultiplexId].ColorArray[1] = RawSample[3].rgb * Weight;
		}
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY)
	{
		// Decode diffuse harmonic
		{
			const uint MultiplexId = 0;

			float Weight = RawSample[0].a;

			if (bNormalizeSample)
			{
				Weight = Weight > 0 ? 1 : 0;
			}

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].SceneColor.rgb = RawSample[0].rgb * Weight;
		}
		
	#if SIGNAL_ARRAY_SIZE >= 2
		// Decode specular harmonic
		{
			const uint MultiplexId = 1;

			float Weight = RawSample[1].a;

			if (bNormalizeSample)
			{
				Weight = Weight > 0 ? 1 : 0;
			}

			OutSamples.Array[MultiplexId].SampleCount = Weight;
			OutSamples.Array[MultiplexId].SceneColor.rgb = RawSample[1].rgb * Weight;
		}
	#endif
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_REFLECTIONS_INPUT)
	{
		OutSamples.Array[0].SampleCount = RawSample[1].r == DENOISER_INVALID_CONFUSION_FACTOR ? 0 : 1;
		OutSamples.Array[0].MissCount = RawSample[1].r == DENOISER_INVALID_CONFUSION_FACTOR ? 0 : (1 - RawSample[0].a);
		OutSamples.Array[0].SceneColor = RawSample[0];

		// max() because ray hit distance is < 0 when miss / not ray has been shot.
		OutSampleFrequencies.Array[0].ConfusionFactor = RawSample[1].r;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r > 0 ? 1 : 0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r;
		}
		
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * (1 - RawSample[0].a);
		OutSamples.Array[0].SceneColor = OutSamples.Array[0].SampleCount * RawSample[0];

		OutSampleFrequencies.Array[0].ConfusionFactor = RawSample[1].g;
	}
	#endif
	#if 1
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_AO_INPUT)
	{
		OutSamples.Array[0].SampleCount = RawSample[1].r < 0 ? 0 : 1;
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * RawSample[0].r;

		OutSampleFrequencies.Array[0].WorldBluringRadius = RawSample[1].r;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_AO_REJECTION)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[0].g > 0 ? 1 : 0;
			OutSamples.Array[0].MissCount = RawSample[0].g > 0 ? RawSample[0].r : 0;
			
		#if SIGNAL_ARRAY_SIZE >= 2
			OutSamples.Array[1].SampleCount = RawSample[0].a > 0 ? 1 : 0;
			OutSamples.Array[1].MissCount = RawSample[0].a > 0 ? RawSample[0].b : 0;
		#endif
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[0].g;
			OutSamples.Array[0].MissCount = RawSample[0].g * RawSample[0].r;
			
		#if SIGNAL_ARRAY_SIZE >= 2
			OutSamples.Array[1].SampleCount = RawSample[0].a;
			OutSamples.Array[1].MissCount = RawSample[0].a * RawSample[0].b;
		#endif
		}
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_AO_HISTORY)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[0].g > 0 ? 1 : 0;
			OutSamples.Array[0].MissCount = RawSample[0].g > 0 ? RawSample[0].r : 0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[0].g;
			OutSamples.Array[0].MissCount = RawSample[0].g * RawSample[0].r;
		}
		
		OutSampleFrequencies.Array[0].WorldBluringRadius = RawSample[0].b;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_INPUT_NSPP)
	{
		OutSamples.Array[0].SampleCount = RawSample[1].g;
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * (1 - RawSample[0].a);
		OutSamples.Array[0].SceneColor = OutSamples.Array[0].SampleCount * RawSample[0];

		OutSampleFrequencies.Array[0].ClosestHitDistance = ((RawSample[1].r == -1.0) ? 1.0e20 : RawSample[1].r);
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r > 0 ? 1 : 0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r;
		}
		
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * (1 - RawSample[0].a);
		OutSamples.Array[0].SceneColor = OutSamples.Array[0].SampleCount * RawSample[0];
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r > 0 ? 1 : 0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[1].r;
		}
		
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * (1 - RawSample[0].a);
		OutSamples.Array[0].SceneColor = OutSamples.Array[0].SampleCount * RawSample[0];
		
		// Temporal analysis.
		#if SIGNAL_ARRAY_SIZE >= 3
			OutSamples.Array[1].SampleCount = OutSamples.Array[0].SampleCount;
			OutSamples.Array[1].SceneColor.x = OutSamples.Array[1].SampleCount * RawSample[1].g;
			OutSamples.Array[2].SampleCount = OutSamples.Array[0].SampleCount;
			OutSamples.Array[2].SceneColor.x = OutSamples.Array[2].SampleCount * RawSample[1].b;
		#endif
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_SSGI_INPUT)
	{
		OutSamples.Array[0].SampleCount = RawSample[0].a;
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * RawSample[1].r;
		OutSamples.Array[0].SceneColor.rgb = OutSamples.Array[0].SampleCount * RawSample[0].rgb;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10)
	{
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[1].g > 0.0 ? 1.0 : 0.0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[1].g * 64.0;
		}
		OutSamples.Array[0].MissCount = OutSamples.Array[0].SampleCount * RawSample[1].r;
		OutSamples.Array[0].SceneColor.rgb = OutSamples.Array[0].SampleCount * RawSample[0].rgb;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY >= 2
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_INPUT)
	{
		OutSamples.Array[0].SampleCount = 1;
		OutSamples.Array[0].ColorArray[0].rgb = OutSamples.Array[0].SampleCount * RawSample[0].rgb;
		OutSamples.Array[0].ColorArray[1].rgb = OutSamples.Array[0].SampleCount * RawSample[1].rgb;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_HISTORY)
	{
		// Diffuse
		if (bNormalizeSample)
		{
			OutSamples.Array[0].SampleCount = RawSample[2].r > 0.0 ? 1.0 : 0.0;
		}
		else
		{
			OutSamples.Array[0].SampleCount = RawSample[2].r * TARGETED_SAMPLE_COUNT;
		}
		OutSamples.Array[0].ColorArray[0].rgb = OutSamples.Array[0].SampleCount * RawSample[0].rgb;
		OutSamples.Array[0].ColorArray[1].rgb = OutSamples.Array[0].SampleCount * RawSample[1].rgb;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_INPUT)
	{
		DecodeLumenDiffuseLighting(RawSample[0], RawSample[1], /* out */ OutSamples.Array[0].ColorSH);
		
		float SampleCount = 1.0; // TODO(Denoiser)
		
		OutSamples.Array[0] = MulSignal(OutSamples.Array[0], SampleCount);
		OutSamples.Array[0].SampleCount = SampleCount;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY)
	{
		DecodeLumenDiffuseLighting(RawSample[0], RawSample[1], /* out */ OutSamples.Array[0].ColorSH);
		
		float SampleCount = RawSample[0].a;
		if (bNormalizeSample)
		{
			SampleCount = SampleCount > 0 ? 1 : 0;
		}
		else
		{
			SampleCount = RawSample[0].a;
		}
		
		OutSamples.Array[0] = MulSignal(OutSamples.Array[0], SampleCount);
		OutSamples.Array[0].SampleCount = SampleCount;
	}
	#endif
} // DecodeMultiplexedSignalsFromFloat4()

void DecodeMultiplexedSignalsFromUint2(
	const uint SignalBufferLayout,
	const uint MultiplexedSampleId,
	const bool bNormalizeSample,
	uint2 RawSample[MAX_MULTIPLEXED_TEXTURES],
	out FSSDSignalArray OutSamples,
	out FSSDSignalFrequencyArray OutSampleFrequencies)
{
	OutSamples = CreateSignalArrayFromScalarValue(0.0);
	OutSampleFrequencies = CreateInvalidSignalFrequencyArray();

	if (0)
	{

	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_PENUMBRA_INJESTION_NSPP)
	{
		UNROLL_N(MAX_SIGNAL_BATCH_SIZE)
		for (uint BatchSignalId = 0; BatchSignalId < MAX_SIGNAL_BATCH_SIZE; BatchSignalId++)
		{
			uint MultiplexId = BatchSignalId;
			uint EncodedData = MultiplexId % 2 ? RawSample[MultiplexId / 2].g : RawSample[MultiplexId / 2].r;

			float MissCountRatio = (EncodedData & 0xFF) / 255.0;
			float TransmissionDistanceRatio = ((EncodedData >> 8) & 0xFF) / 255.0;
			float WorldBluringRadius = f16tof32(EncodedData >> 16);
			float SampleCount = (WorldBluringRadius == -2.0 ? 0.0 : 1.0);

			float MissCount = SampleCount * MissCountRatio;
			float TransmissionDistance = TransmissionDistanceRatio;

			// TODO(Denoiser): feed the actual number of sample.
			OutSamples.Array[MultiplexId].SampleCount = SampleCount;
			OutSamples.Array[MultiplexId].MissCount = MissCount;
			OutSamples.Array[MultiplexId].TransmissionDistance = TransmissionDistance;

			OutSampleFrequencies.Array[MultiplexId].WorldBluringRadius = WorldBluringRadius;
		}
	}
}

/** Decode input signal sample from raw float. */
void DecodeMultiplexedSignals(
	const uint SignalBufferLayout,
	const uint MultiplexedSampleId,
	const bool bNormalizeSample,
	FSSDCompressedMultiplexedSample CompressedSample,
	out FSSDSignalArray OutSamples,
	out FSSDSignalFrequencyArray OutSampleFrequencies)
#if CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
{
	DecodeMultiplexedSignalsFromFloat4(
		SignalBufferLayout,
		MultiplexedSampleId,
		bNormalizeSample,
		CompressedSample.VGPRArray,
		/* out */ OutSamples,
		/* out */ OutSampleFrequencies);
} // DecodeMultiplexedSignals()
#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT2
{
	FSSDSignalArray MultiplexedSamples = CreateSignalArrayFromScalarValue(0.0);
	DecodeMultiplexedSignalsFromUint2(
		SignalBufferLayout,
		MultiplexedSampleId,
		bNormalizeSample,
		CompressedSample.VGPRArray,
		/* out */ OutSamples,
		/* out */ OutSampleFrequencies);
} // DecodeMultiplexedSignals()
#endif

/** Encode output signal sample. */
void EncodeMultiplexedSignals(
	const uint SignalBufferLayout, const uint MultiplexCount,
	FSSDSignalSample Sample[SIGNAL_ARRAY_SIZE],
	FSSDSignalFrequency SampleFrequency[SIGNAL_ARRAY_SIZE],
	out FSSDOutputRawSample OutRawSample[MAX_MULTIPLEXED_TEXTURES],
	out uint OutBufferCount)
#if CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
{
	// Init all raw samples.
	UNROLL_N(MAX_MULTIPLEXED_TEXTURES)
	for (uint i = 0; i < MAX_MULTIPLEXED_TEXTURES; i++)
		OutRawSample[i] = 0;

	// Number of buffer the signal get encoded onto <= MAX_MULTIPLEXED_TEXTURES.
	OutBufferCount = 1;

	if (0)
	{
		// NOP
	}
	#if 1
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY)
	{
		UNROLL
		for (uint MultiplexId = 0; MultiplexId < MultiplexCount; MultiplexId++)
		{
			float NormalizationFactor = SafeRcp(Sample[MultiplexId].SampleCount);

			float NormalizedTransmissionDistance = Sample[MultiplexId].TransmissionDistance * NormalizationFactor;

			OutRawSample[MultiplexId] = float4(
				Sample[MultiplexId].MissCount * NormalizationFactor,
				Sample[MultiplexId].SampleCount,
				SampleFrequency[MultiplexId].WorldBluringRadius,
				NormalizedTransmissionDistance);
		}
		OutBufferCount = MultiplexCount;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY >= 2 && SIGNAL_ARRAY_SIZE >= 2
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_RECONSTRUCTION)
	{
		// diffuse harmonic
		{
			const uint MultiplexId = 0;

			float NormalizationFactor = SafeRcp(Sample[MultiplexId].SampleCount);

			OutRawSample[0].rgb = Sample[MultiplexId].ColorArray[0] * NormalizationFactor;
			OutRawSample[0].a = Sample[MultiplexId].SampleCount;
			OutRawSample[1].rgb = Sample[MultiplexId].ColorArray[1] * NormalizationFactor;
		}
		
		// specular harmonic
		{
			const uint MultiplexId = 1;

			float NormalizationFactor = SafeRcp(Sample[1].SampleCount);
			
			OutRawSample[2].rgb = Sample[MultiplexId].ColorArray[0] * NormalizationFactor;
			OutRawSample[2].a = Sample[MultiplexId].SampleCount;
			OutRawSample[3].rgb = Sample[MultiplexId].ColorArray[1] * NormalizationFactor;
		}

		OutBufferCount = 4;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY)
	{
		// Encode diffuse harmonic
		{
			const uint MultiplexId = 0;

			float NormalizationFactor = SafeRcp(Sample[MultiplexId].SampleCount);

			OutRawSample[0].rgb = ClampColorForEncoding(Sample[MultiplexId].SceneColor.rgb * NormalizationFactor);
			OutRawSample[0].a = Sample[MultiplexId].SampleCount;
		}

		// Encode specular harmonic
	#if SIGNAL_ARRAY_SIZE >= 2
		{
			const uint MultiplexId = 1;

			float NormalizationFactor = SafeRcp(Sample[MultiplexId].SampleCount);

			OutRawSample[1].rgb = ClampColorForEncoding(Sample[MultiplexId].SceneColor.rgb * NormalizationFactor);
			OutRawSample[1].a = Sample[MultiplexId].SampleCount;
		}
	#endif

		OutBufferCount = 2;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY)
	{
		float NormalizationFactor = SafeRcp(Sample[0].SampleCount);

		// Outputs scene color to be compatible with a SSR output.
		OutRawSample[0] = ClampColorForEncoding(Sample[0].SceneColor * NormalizationFactor);
		OutRawSample[1].r = Sample[0].SampleCount;
		OutRawSample[1].g = SampleFrequency[0].ConfusionFactor;
		OutBufferCount = 2;
	}
	#endif
	#if 1
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_AO_REJECTION)
	{
		// Outputs number of ray miss in red to be compatible as a SSAO output.
		OutRawSample[0].r = Sample[0].SampleCount > 0 ? Sample[0].MissCount / Sample[0].SampleCount : 1.0;
		OutRawSample[0].g = Sample[0].SampleCount;
		
	#if SIGNAL_ARRAY_SIZE >= 2
		if (MultiplexCount == 2)
		{
			OutRawSample[0].b = Sample[1].SampleCount > 0 ? Sample[1].MissCount / Sample[1].SampleCount : 1.0;
			OutRawSample[0].a = Sample[1].SampleCount;
		}
	#endif
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_AO_HISTORY)
	{
		// Outputs number of ray miss in red to be compatible as a SSAO output.
		OutRawSample[0].r = Sample[0].SampleCount > 0 ? Sample[0].MissCount / Sample[0].SampleCount : 1.0;
		OutRawSample[0].g = Sample[0].SampleCount;
		OutRawSample[0].b = SampleFrequency[0].WorldBluringRadius;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION)
	{
		float NormalizationFactor = SafeRcp(Sample[0].SampleCount);

		OutRawSample[0] = ClampColorForEncoding(Sample[0].SceneColor * NormalizationFactor);
		OutRawSample[1].r = Sample[0].SampleCount;
		OutBufferCount = 2;
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY)
	{
		float NormalizationFactor = SafeRcp(Sample[0].SampleCount);

		OutRawSample[0] = ClampColorForEncoding(Sample[0].SceneColor * NormalizationFactor);
		OutRawSample[1].r = Sample[0].SampleCount;
		OutBufferCount = 2;

		// Temporal analysis.
		#if SIGNAL_ARRAY_SIZE >= 3
		if (MultiplexCount == 3)
		{
			OutRawSample[1].g = Sample[1].SceneColor.x * NormalizationFactor;
			OutRawSample[1].b = Sample[2].SceneColor.x * NormalizationFactor;
		}
		#endif
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10)
	{
		float NormalizationFactor = SafeRcp(Sample[0].SampleCount);

		OutRawSample[0].rgb = ClampColorForEncoding(Sample[0].SceneColor * NormalizationFactor).rgb;
		OutRawSample[1].g = saturate(Sample[0].SampleCount * rcp(64.0));
		OutRawSample[1].r = Sample[0].MissCount * NormalizationFactor;
		OutBufferCount = 2;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY >= 2
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_HISTORY)
	{
		float NormalizationFactor = SafeRcp(Sample[0].SampleCount);
		OutRawSample[0].rgb = ClampColorForEncoding(Sample[0].ColorArray[0] * NormalizationFactor);
		OutRawSample[1].rgb = ClampColorForEncoding(Sample[0].ColorArray[1] * NormalizationFactor);
		OutRawSample[2].r = saturate(Sample[0].SampleCount * rcp(TARGETED_SAMPLE_COUNT));

		OutBufferCount = 3;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY)
	{
		FSSDSignalSample NormalizedSample = NormalizeToOneSample(Sample[0]);
		//FSSDSignalSample NormalizedSample = (Sample[0]);

		EncodeLumenDiffuseLighting(NormalizedSample.ColorSH, /* out */ OutRawSample[0], /* out */ OutRawSample[1]);
		OutRawSample[0].a = Sample[0].SampleCount;
		OutBufferCount = 2;
	}
	#endif
} // EncodeMultiplexedSignals()
#elif CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_UINT2
{
	// Init all raw samples.
	UNROLL_N(MAX_MULTIPLEXED_TEXTURES)
	for (uint i = 0; i < MAX_MULTIPLEXED_TEXTURES; i++)
		OutRawSample[i] = 0;

	// Number of buffer the signal get encoded onto <= MAX_MULTIPLEXED_TEXTURES.
	OutBufferCount = 1;

	if (0)
	{
	}
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_PENUMBRA_INJESTION_NSPP)
	{
		UNROLL
		for (uint MultiplexId = 0; MultiplexId < MultiplexCount; MultiplexId++)
		{
			float MissCount = Sample[MultiplexId].MissCount;
			float WorldBluringRadius = SampleFrequency[MultiplexId].WorldBluringRadius;
			float TransmissionDistance = Sample[MultiplexId].TransmissionDistance;
			float SampleCount = Sample[MultiplexId].SampleCount;
			if (SampleCount == 0)
			{
				WorldBluringRadius = -2.0;
			}

			float MissCountRatio = MissCount * SafeRcp(SampleCount);
			float TransmissionDistanceRatio = TransmissionDistance;

			uint EncodedData = MissCountRatio * 255;
			EncodedData |= (uint(TransmissionDistanceRatio * 255) << 8);
			EncodedData |= (f32tof16(WorldBluringRadius) << 16);

			if (MultiplexId % 2)
			{
				OutRawSample[MultiplexId / 2].g = EncodedData;
			}
			else
			{
				OutRawSample[MultiplexId / 2].r = EncodedData;
			}
		}
		OutBufferCount = (MultiplexCount + 1) / 2;
	}
	#if COMPILE_SIGNAL_COLOR_SH
	else if (SignalBufferLayout == SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC)
	{
		const uint EncodeOptions = (CONFIG_ENCODING_CLAMP_COLOR ? SSD_ENCODE_CLAMP_COLOR : 0) | SSD_ENCODE_NORMALIZE;

		EncodeDiffuseSphericalHarmonicTexel(
			Sample[0].SampleCount,
			Sample[0].MissCount,
			Sample[0].ColorSH,
			EncodeOptions,
			/* out */ OutRawSample);

		OutBufferCount = 4;
	}
	#endif
} // EncodeMultiplexedSignals()
#else
	#error Unimplemented.
#endif


/** Sample the raw of multiple input signals that have been multiplexed. */
FSSDCompressedMultiplexedSample SampleCompressedMultiplexedSignals(
	FSSDTexture2D SignalBuffer0, FSSDTexture2D SignalBuffer1, FSSDTexture2D SignalBuffer2, FSSDTexture2D SignalBuffer3,
	SamplerState Sampler, float2 UV, uint2 PixelCoord)
{
	FSSDCompressedMultiplexedSample CompressedSample;

	// Isolate the texture fetches to force lattency hiding, to outsmart compilers that tries to
	// discard some of the texture fetches for instance when SampleCount == 0 in another texture.
	ISOLATE
	{
		#if CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
		{
			CompressedSample.VGPRArray[0] = SignalBuffer0.SampleLevel(Sampler, UV, 0.0);
			CompressedSample.VGPRArray[1] = SignalBuffer1.SampleLevel(Sampler, UV, 0.0);
			CompressedSample.VGPRArray[2] = SignalBuffer2.SampleLevel(Sampler, UV, 0.0);
			CompressedSample.VGPRArray[3] = SignalBuffer3.SampleLevel(Sampler, UV, 0.0);
		}
		#elif CONFIG_SIGNAL_INPUT_TEXTURE_TYPE >= SIGNAL_TEXTURE_TYPE_UINT1 && CONFIG_SIGNAL_INPUT_TEXTURE_TYPE <= SIGNAL_TEXTURE_TYPE_UINT4
		{
			// TODO(Denoiser): Exposed the int3 instead as function parameter.
			int3 Coord = int3(PixelCoord, 0);
			CompressedSample.VGPRArray[0] = SignalBuffer0.Load(Coord);
			CompressedSample.VGPRArray[1] = SignalBuffer1.Load(Coord);
			CompressedSample.VGPRArray[2] = SignalBuffer2.Load(Coord);
			CompressedSample.VGPRArray[3] = SignalBuffer3.Load(Coord);
		}
		#else
			#error Unimplemented.
		#endif
	}

	return CompressedSample;
}


/** Sample multiple input signals that have been multiplexed. */
void SampleMultiplexedSignals( // TODO
	FSSDTexture2D SignalBuffer0, FSSDTexture2D SignalBuffer1, FSSDTexture2D SignalBuffer2, FSSDTexture2D SignalBuffer3,
	SamplerState Sampler,
	const uint SignalBufferLayout, const uint MultiplexedSampleId,
	const bool bNormalizeSample,
	float2 UV,
	out FSSDSignalArray OutMultiplexedSamples,
	out FSSDSignalFrequencyArray OutMultiplexedFrequencies)
{
	uint2 PixelCoord = BufferUVToBufferPixelCoord(UV);

	FSSDCompressedMultiplexedSample CompressedSample = SampleCompressedMultiplexedSignals(
		SignalBuffer0, SignalBuffer1, SignalBuffer2, SignalBuffer3,
		Sampler, UV, PixelCoord);

	DecodeMultiplexedSignals(
		SignalBufferLayout, MultiplexedSampleId, bNormalizeSample, CompressedSample,
		/* out */ OutMultiplexedSamples,
		/* out */ OutMultiplexedFrequencies);
}


/** Outputs multiplexed signal. */
void OutputMultiplexedSignal(
	FSSDRWTexture2D OutputSignalBuffer0,
	FSSDRWTexture2D OutputSignalBuffer1,
	FSSDRWTexture2D OutputSignalBuffer2,
	FSSDRWTexture2D OutputSignalBuffer3,
	const uint SignalBufferLayout, const uint MultiplexCount,
	const uint2 PixelPosition,
	FSSDSignalArray MultiplexedSamples,
	FSSDSignalFrequencyArray MultiplexedFrequencies)
{
	// Encode the output signal.
	FSSDOutputRawSample RawSample[MAX_MULTIPLEXED_TEXTURES];
	uint BufferCount;
	EncodeMultiplexedSignals(
		SignalBufferLayout, MultiplexCount,
		MultiplexedSamples.Array,
		MultiplexedFrequencies.Array,
		/* out */ RawSample, /* out */ BufferCount);

	// Output the raw encoded sample according to number of RT they requires.
	if (BufferCount >= 1)
		OutputSignalBuffer0[PixelPosition] = RawSample[0];
	if (BufferCount >= 2)
		OutputSignalBuffer1[PixelPosition] = RawSample[1];
	if (BufferCount >= 3)
		OutputSignalBuffer2[PixelPosition] = RawSample[2];
	if (BufferCount >= 4)
		OutputSignalBuffer3[PixelPosition] = RawSample[3];
}
