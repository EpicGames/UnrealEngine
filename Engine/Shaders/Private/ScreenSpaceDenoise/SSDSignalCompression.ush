// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDSignalCore.ush"
#include "SSDPublicBufferEncoding.ush"


//------------------------------------------------------- STRUCTURE

#if COMPILE_SIGNAL_COLOR_SH
	#define MAX_SIGNAL_COMPRESSED_VGPRS 7
#else
	#define MAX_SIGNAL_COMPRESSED_VGPRS 1
#endif

/** Generic data structure to store a signal sample in a compressed form. */
struct FSSDCompressedSignalSample
{
	// Raw array of vector general purpose registers used to store signal.
	uint VGPR[MAX_SIGNAL_COMPRESSED_VGPRS];
};


/** Compress part of signal into smaller VGPR foorprint. */
void CompressSignalSample(in FSSDSignalSample Sample, uint CompressionLayout, out FSSDCompressedSignalSample OutCompressedSample)
{
	UNROLL_N(MAX_SIGNAL_COMPRESSED_VGPRS)
	for (uint i = 0; i < MAX_SIGNAL_COMPRESSED_VGPRS; i++)
	{
		OutCompressedSample.VGPR[i] = 0;
	}

	if (CompressionLayout == SIGNAL_COMPRESSION_DISABLED || DEBUG_DISABLE_SIGNAL_COMPRESSION)
	{
		// NOP
	}
	#if COMPILE_SIGNAL_COLOR_SH
	else if (CompressionLayout == SIGNAL_COMPRESSION_DIFFUSE_INDIRECT_HARMONIC)
	{
		OutCompressedSample.VGPR[0] = Compress2Floats(float2(Sample.SampleCount, Sample.MissCount));
		#if SPHERICAL_HARMONIC_ORDER == 2
			OutCompressedSample.VGPR[1] = Compress2Floats(Sample.ColorSH.R.V.xy);
			OutCompressedSample.VGPR[2] = Compress2Floats(Sample.ColorSH.R.V.zw);
			OutCompressedSample.VGPR[3] = Compress2Floats(Sample.ColorSH.G.V.xy);
			OutCompressedSample.VGPR[4] = Compress2Floats(Sample.ColorSH.G.V.zw);
			OutCompressedSample.VGPR[5] = Compress2Floats(Sample.ColorSH.B.V.xy);
			OutCompressedSample.VGPR[6] = Compress2Floats(Sample.ColorSH.B.V.zw);
		#else
			#error Unimplemented.
		#endif
	}
	#endif
}

/** Decompress signal from tight VGPR layout into a sample. */
void UncompressSignalSample(in FSSDCompressedSignalSample CompressedSample, uint CompressionLayout, inout FSSDSignalSample OutSample)
{
	if (CompressionLayout == SIGNAL_COMPRESSION_DISABLED || DEBUG_DISABLE_SIGNAL_COMPRESSION)
	{
		// NOP to not clobber information in OutSample that remained uncompressed.
	}
	#if COMPILE_SIGNAL_COLOR_SH
	else if (CompressionLayout == SIGNAL_COMPRESSION_DIFFUSE_INDIRECT_HARMONIC)
	{
		OutSample.SampleCount    = Uncompress2Floats(CompressedSample.VGPR[0]).x;
		OutSample.MissCount      = Uncompress2Floats(CompressedSample.VGPR[0]).y;
		#if SPHERICAL_HARMONIC_ORDER == 2
			OutSample.ColorSH.R.V.xy = Uncompress2Floats(CompressedSample.VGPR[1]);
			OutSample.ColorSH.R.V.zw = Uncompress2Floats(CompressedSample.VGPR[2]);
			OutSample.ColorSH.G.V.xy = Uncompress2Floats(CompressedSample.VGPR[3]);
			OutSample.ColorSH.G.V.zw = Uncompress2Floats(CompressedSample.VGPR[4]);
			OutSample.ColorSH.B.V.xy = Uncompress2Floats(CompressedSample.VGPR[5]);
			OutSample.ColorSH.B.V.zw = Uncompress2Floats(CompressedSample.VGPR[6]);
		#else
			#error Unimplemented.
		#endif
	}
	#endif
}

FSSDCompressedSignalSample CreateCompressedSignalSampleFromScalarValue(float Scalar, uint CompressionLayout)
{
	FSSDSignalSample Sample = CreateSignalSampleFromScalarValue(Scalar);

	FSSDCompressedSignalSample CompressedSample;
	CompressSignalSample(Sample, CompressionLayout, /* out */ CompressedSample);
	return CompressedSample;
}
