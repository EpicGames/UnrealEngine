// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSDCommon.ush"


#ifndef COMPILE_SIGNAL_COLOR
	#define COMPILE_SIGNAL_COLOR 0
#endif

#ifndef COMPILE_SIGNAL_COLOR_SH
	#define COMPILE_SIGNAL_COLOR_SH 0
#endif

#ifndef COMPILE_SIGNAL_COLOR_ARRAY
	#define COMPILE_SIGNAL_COLOR_ARRAY 0
#endif

// Maximum number of indepedendent signal domain that can be denoised at same time.
#ifndef MAX_SIGNAL_BATCH_SIZE
	#error Need to specify the number of signal domain being denoised.
#endif

// Number of sample that can be multiplexed into FSSDSignalSample.
#ifndef SIGNAL_ARRAY_SIZE
	#error Need to specify the size of the signal array.
#endif


//------------------------------------------------------- STRUCTURE

/** Generic linearily filterable data structure to manipulate any kind of signal. */
struct FSSDSignalSample
{
	// Number of sample accumulated.
	float SampleCount;
	
	// Scene color and alpha.
	#if COMPILE_SIGNAL_COLOR
		float4 SceneColor;
	#endif

	// Array of colors.
	#if COMPILE_SIGNAL_COLOR_ARRAY
		float3 ColorArray[COMPILE_SIGNAL_COLOR_ARRAY];
	#endif

	// Spherical harmonic of the color.
	#if COMPILE_SIGNAL_COLOR_SH
		FSSDSphericalHarmonicRGB ColorSH;
	#endif

	// Number of ray that did not find any intersections.
	float MissCount;

	float TransmissionDistance;
};

/** Generic data structure about the frequency of a FSSDSignalSample. */
struct FSSDSignalFrequency
{
	// Hit distance of a this sample.
	float ClosestHitDistance;

	// Blur radius allowed for this sample.
	float WorldBluringRadius;

	// Circle confusion multiplier.
	float ConfusionFactor;
};

/** Array of signal samples. Technically should be a typedef, but there is a bug in HLSL with the out operator of an array. */
struct FSSDSignalArray
{
	FSSDSignalSample Array[SIGNAL_ARRAY_SIZE];
};

/** Array of signal frequencies. Technically should be a typedef, but there is a bug in HLSL with the out operator of an array. */
struct FSSDSignalFrequencyArray
{
	FSSDSignalFrequency Array[SIGNAL_ARRAY_SIZE];
};


FSSDSignalSample CreateSignalSampleFromScalarValue(float Scalar)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor = Scalar;
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = Scalar;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		OutSample.ColorSH.R.V = Scalar;
		OutSample.ColorSH.G.V = Scalar;
		OutSample.ColorSH.B.V = Scalar;
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		OutSample.ColorSH.R.V0 = Scalar;
		OutSample.ColorSH.R.V1 = Scalar;
		OutSample.ColorSH.R.V2 = Scalar;
		OutSample.ColorSH.G.V0 = Scalar;
		OutSample.ColorSH.G.V1 = Scalar;
		OutSample.ColorSH.G.V2 = Scalar;
		OutSample.ColorSH.B.V0 = Scalar;
		OutSample.ColorSH.B.V1 = Scalar;
		OutSample.ColorSH.B.V2 = Scalar;
	#endif
	OutSample.SampleCount = Scalar;
	OutSample.MissCount = Scalar;
	OutSample.TransmissionDistance = Scalar;
	return OutSample;
}

FSSDSignalFrequency CreateInvalidSignalFrequency()
{
	FSSDSignalFrequency Frequency;
	Frequency.ClosestHitDistance = DENOISER_INVALID_HIT_DISTANCE;
	Frequency.WorldBluringRadius = WORLD_RADIUS_INVALID;
	Frequency.ConfusionFactor = CONFUSION_FACTOR_INVALID;
	return Frequency;
}

FSSDSignalSample CreateSignalSample()
{
	return CreateSignalSampleFromScalarValue(0.0);
}

FSSDSignalArray CreateSignalArrayFromScalarValue(float Scalar)
{
	FSSDSignalArray OutSamples;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutSamples.Array[BatchedSignalId] = CreateSignalSampleFromScalarValue(Scalar);
	}
	return OutSamples;
}

FSSDSignalFrequencyArray CreateInvalidSignalFrequencyArray()
{
	FSSDSignalFrequencyArray OutFrequencies;
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint BatchedSignalId = 0; BatchedSignalId < SIGNAL_ARRAY_SIZE; BatchedSignalId++)
	{
		OutFrequencies.Array[BatchedSignalId] = CreateInvalidSignalFrequency();
	}
	return OutFrequencies;
}


//------------------------------------------------------- SIGNAL OPERATORS

FSSDSignalSample MulSignal(FSSDSignalSample Sample, float Scalar)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor         = Sample.SceneColor * Scalar;
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = Sample.ColorArray[ColorId] * Scalar;
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH
		OutSample.ColorSH            = MulSH(Sample.ColorSH, Scalar);
	#endif
	OutSample.SampleCount            = Sample.SampleCount * Scalar;
	OutSample.MissCount              = Sample.MissCount * Scalar;
	OutSample.TransmissionDistance   = Sample.TransmissionDistance * Scalar;
	return OutSample;
}

FSSDSignalSample AddSignal(FSSDSignalSample SampleA, FSSDSignalSample SampleB)
{
	#if COMPILE_SIGNAL_COLOR
		SampleA.SceneColor         += SampleB.SceneColor;
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			SampleA.ColorArray[ColorId] += SampleB.ColorArray[ColorId];
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH
		SampleA.ColorSH            = SSDAddSH(SampleA.ColorSH, SampleB.ColorSH);
	#endif
	SampleA.SampleCount            += SampleB.SampleCount;
	SampleA.MissCount              += SampleB.MissCount;
	SampleA.TransmissionDistance   += SampleB.TransmissionDistance;
	return SampleA;
}

FSSDSignalSample MinusSignal(FSSDSignalSample Sample)
{
	#if COMPILE_SIGNAL_COLOR
		Sample.SceneColor         = -Sample.SceneColor;
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			Sample.ColorArray[ColorId] = -Sample.ColorArray[ColorId];
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH
		Sample.ColorSH         = MulSH(Sample.ColorSH, -1.0);
	#endif
	Sample.SampleCount            = -Sample.SampleCount;
	Sample.MissCount              = -Sample.MissCount;
	Sample.TransmissionDistance   = -Sample.TransmissionDistance;
	return Sample;
}

FSSDSignalSample SubtractSignal(FSSDSignalSample SampleA, FSSDSignalSample SampleB)
{
	return AddSignal(SampleA, MinusSignal(SampleB));
}

FSSDSignalSample AbsSignal(FSSDSignalSample Sample)
{
	#if COMPILE_SIGNAL_COLOR
		Sample.SceneColor         = abs(Sample.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			Sample.ColorArray[ColorId] = abs(Sample.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		Sample.ColorSH.R.V = abs(Sample.ColorSH.R.V);
		Sample.ColorSH.G.V = abs(Sample.ColorSH.G.V);
		Sample.ColorSH.B.V = abs(Sample.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		Sample.ColorSH.R.V0 = abs(Sample.ColorSH.R.V0);
		Sample.ColorSH.R.V1 = abs(Sample.ColorSH.R.V1);
		Sample.ColorSH.R.V2 = abs(Sample.ColorSH.R.V2);
		Sample.ColorSH.G.V0 = abs(Sample.ColorSH.G.V0);
		Sample.ColorSH.G.V1 = abs(Sample.ColorSH.G.V1);
		Sample.ColorSH.G.V2 = abs(Sample.ColorSH.G.V2);
		Sample.ColorSH.B.V0 = abs(Sample.ColorSH.B.V0);
		Sample.ColorSH.B.V1 = abs(Sample.ColorSH.B.V1);
		Sample.ColorSH.B.V2 = abs(Sample.ColorSH.B.V2);
	#endif
	Sample.SampleCount            = abs(Sample.SampleCount);
	Sample.MissCount              = abs(Sample.MissCount);
	Sample.TransmissionDistance = abs(Sample.TransmissionDistance);
	return Sample;
}

FSSDSignalSample SqrtSignal(FSSDSignalSample Sample)
{
	#if COMPILE_SIGNAL_COLOR
		Sample.SceneColor         = sqrt(Sample.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			Sample.ColorArray[ColorId] = sqrt(Sample.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		Sample.ColorSH.R.V = sqrt(Sample.ColorSH.R.V);
		Sample.ColorSH.G.V = sqrt(Sample.ColorSH.G.V);
		Sample.ColorSH.B.V = sqrt(Sample.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		Sample.ColorSH.R.V0 = sqrt(Sample.ColorSH.R.V0);
		Sample.ColorSH.R.V1 = sqrt(Sample.ColorSH.R.V1);
		Sample.ColorSH.R.V2 = sqrt(Sample.ColorSH.R.V2);
		Sample.ColorSH.G.V0 = sqrt(Sample.ColorSH.G.V0);
		Sample.ColorSH.G.V1 = sqrt(Sample.ColorSH.G.V1);
		Sample.ColorSH.G.V2 = sqrt(Sample.ColorSH.G.V2);
		Sample.ColorSH.B.V0 = sqrt(Sample.ColorSH.B.V0);
		Sample.ColorSH.B.V1 = sqrt(Sample.ColorSH.B.V1);
		Sample.ColorSH.B.V2 = sqrt(Sample.ColorSH.B.V2);
	#endif
	Sample.SampleCount            = sqrt(Sample.SampleCount);
	Sample.MissCount              = sqrt(Sample.MissCount);
	Sample.TransmissionDistance   = sqrt(Sample.TransmissionDistance);
	return Sample;
}

FSSDSignalSample PowerSignal(FSSDSignalSample Sample, float Exponent)
{
	#if COMPILE_SIGNAL_COLOR
		Sample.SceneColor.r       = pow(Sample.SceneColor.r, Exponent);
		Sample.SceneColor.g       = pow(Sample.SceneColor.g, Exponent);
		Sample.SceneColor.b       = pow(Sample.SceneColor.b, Exponent);
		Sample.SceneColor.a       = pow(Sample.SceneColor.a, Exponent);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			Sample.ColorArray[ColorId] = pow(Sample.ColorArray[ColorId], Exponent);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		Sample.ColorSH.R.V = pow(Sample.ColorSH.R.V, Exponent);
		Sample.ColorSH.G.V = pow(Sample.ColorSH.G.V, Exponent);
		Sample.ColorSH.B.V = pow(Sample.ColorSH.B.V, Exponent);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		Sample.ColorSH.R.V0 = pow(Sample.ColorSH.R.V0, Exponent);
		Sample.ColorSH.R.V1 = pow(Sample.ColorSH.R.V1, Exponent);
		Sample.ColorSH.R.V2 = pow(Sample.ColorSH.R.V2, Exponent);
		Sample.ColorSH.G.V0 = pow(Sample.ColorSH.G.V0, Exponent);
		Sample.ColorSH.G.V1 = pow(Sample.ColorSH.G.V1, Exponent);
		Sample.ColorSH.G.V2 = pow(Sample.ColorSH.G.V2, Exponent);
		Sample.ColorSH.B.V0 = pow(Sample.ColorSH.B.V0, Exponent);
		Sample.ColorSH.B.V1 = pow(Sample.ColorSH.B.V1, Exponent);
		Sample.ColorSH.B.V2 = pow(Sample.ColorSH.B.V2, Exponent);
	#endif
	Sample.SampleCount            = pow(Sample.SampleCount, Exponent);
	Sample.MissCount              = pow(Sample.MissCount, Exponent);
	Sample.TransmissionDistance   = pow(Sample.TransmissionDistance, Exponent);
	return Sample;
}

FSSDSignalSample MinSignal(FSSDSignalSample SampleA, FSSDSignalSample SampleB)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor = min(SampleA.SceneColor, SampleB.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = min(SampleA.ColorArray[ColorId], SampleB.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		OutSample.ColorSH.R.V = min(SampleA.ColorSH.R.V, SampleB.ColorSH.R.V);
		OutSample.ColorSH.G.V = min(SampleA.ColorSH.G.V, SampleB.ColorSH.G.V);
		OutSample.ColorSH.B.V = min(SampleA.ColorSH.B.V, SampleB.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		OutSample.ColorSH.R.V0 = min(SampleA.ColorSH.R.V0, SampleB.ColorSH.R.V0);
		OutSample.ColorSH.R.V1 = min(SampleA.ColorSH.R.V1, SampleB.ColorSH.R.V1);
		OutSample.ColorSH.R.V2 = min(SampleA.ColorSH.R.V2, SampleB.ColorSH.R.V2);
		OutSample.ColorSH.G.V0 = min(SampleA.ColorSH.G.V0, SampleB.ColorSH.G.V0);
		OutSample.ColorSH.G.V1 = min(SampleA.ColorSH.G.V1, SampleB.ColorSH.G.V1);
		OutSample.ColorSH.G.V2 = min(SampleA.ColorSH.G.V2, SampleB.ColorSH.G.V2);
		OutSample.ColorSH.B.V0 = min(SampleA.ColorSH.B.V0, SampleB.ColorSH.B.V0);
		OutSample.ColorSH.B.V1 = min(SampleA.ColorSH.B.V1, SampleB.ColorSH.B.V1);
		OutSample.ColorSH.B.V2 = min(SampleA.ColorSH.B.V2, SampleB.ColorSH.B.V2);
	#endif
	OutSample.SampleCount = min(SampleA.SampleCount, SampleB.SampleCount);
	OutSample.MissCount = min(SampleA.MissCount, SampleB.MissCount);
	OutSample.TransmissionDistance = min(SampleA.TransmissionDistance, SampleB.TransmissionDistance);
	return OutSample;
}

FSSDSignalSample MaxSignal(FSSDSignalSample SampleA, FSSDSignalSample SampleB)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor = max(SampleA.SceneColor, SampleB.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = max(SampleA.ColorArray[ColorId], SampleB.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		OutSample.ColorSH.R.V = max(SampleA.ColorSH.R.V, SampleB.ColorSH.R.V);
		OutSample.ColorSH.G.V = max(SampleA.ColorSH.G.V, SampleB.ColorSH.G.V);
		OutSample.ColorSH.B.V = max(SampleA.ColorSH.B.V, SampleB.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		OutSample.ColorSH.R.V0 = max(SampleA.ColorSH.R.V0, SampleB.ColorSH.R.V0);
		OutSample.ColorSH.R.V1 = max(SampleA.ColorSH.R.V1, SampleB.ColorSH.R.V1);
		OutSample.ColorSH.R.V2 = max(SampleA.ColorSH.R.V2, SampleB.ColorSH.R.V2);
		OutSample.ColorSH.G.V0 = max(SampleA.ColorSH.G.V0, SampleB.ColorSH.G.V0);
		OutSample.ColorSH.G.V1 = max(SampleA.ColorSH.G.V1, SampleB.ColorSH.G.V1);
		OutSample.ColorSH.G.V2 = max(SampleA.ColorSH.G.V2, SampleB.ColorSH.G.V2);
		OutSample.ColorSH.B.V0 = max(SampleA.ColorSH.B.V0, SampleB.ColorSH.B.V0);
		OutSample.ColorSH.B.V1 = max(SampleA.ColorSH.B.V1, SampleB.ColorSH.B.V1);
		OutSample.ColorSH.B.V2 = max(SampleA.ColorSH.B.V2, SampleB.ColorSH.B.V2);
	#endif
	OutSample.SampleCount = max(SampleA.SampleCount, SampleB.SampleCount);
	OutSample.MissCount = max(SampleA.MissCount, SampleB.MissCount);
	OutSample.TransmissionDistance = max(SampleA.TransmissionDistance, SampleB.TransmissionDistance);
	return OutSample;
}

FSSDSignalSample ClampSignal(FSSDSignalSample Sample, FSSDSignalSample SampleMin, FSSDSignalSample SampleMax)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor = clamp(Sample.SceneColor, SampleMin.SceneColor, SampleMax.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = clamp(Sample.ColorArray[ColorId], SampleMin.ColorArray[ColorId], SampleMax.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		OutSample.ColorSH.R.V = clamp(Sample.ColorSH.R.V, SampleMin.ColorSH.R.V, SampleMax.ColorSH.R.V);
		OutSample.ColorSH.G.V = clamp(Sample.ColorSH.G.V, SampleMin.ColorSH.G.V, SampleMax.ColorSH.G.V);
		OutSample.ColorSH.B.V = clamp(Sample.ColorSH.B.V, SampleMin.ColorSH.B.V, SampleMax.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		OutSample.ColorSH.R.V0 = clamp(Sample.ColorSH.R.V0, SampleMin.ColorSH.R.V0, SampleMax.ColorSH.R.V0);
		OutSample.ColorSH.R.V1 = clamp(Sample.ColorSH.R.V1, SampleMin.ColorSH.R.V1, SampleMax.ColorSH.R.V1);
		OutSample.ColorSH.R.V2 = clamp(Sample.ColorSH.R.V2, SampleMin.ColorSH.R.V2, SampleMax.ColorSH.R.V2);
		OutSample.ColorSH.G.V0 = clamp(Sample.ColorSH.G.V0, SampleMin.ColorSH.G.V0, SampleMax.ColorSH.G.V0);
		OutSample.ColorSH.G.V1 = clamp(Sample.ColorSH.G.V1, SampleMin.ColorSH.G.V1, SampleMax.ColorSH.G.V1);
		OutSample.ColorSH.G.V2 = clamp(Sample.ColorSH.G.V2, SampleMin.ColorSH.G.V2, SampleMax.ColorSH.G.V2);
		OutSample.ColorSH.B.V0 = clamp(Sample.ColorSH.B.V0, SampleMin.ColorSH.B.V0, SampleMax.ColorSH.B.V0);
		OutSample.ColorSH.B.V1 = clamp(Sample.ColorSH.B.V1, SampleMin.ColorSH.B.V1, SampleMax.ColorSH.B.V1);
		OutSample.ColorSH.B.V2 = clamp(Sample.ColorSH.B.V2, SampleMin.ColorSH.B.V2, SampleMax.ColorSH.B.V2);
	#endif
	OutSample.SampleCount = clamp(Sample.SampleCount, SampleMin.SampleCount, SampleMax.SampleCount);
	OutSample.MissCount = clamp(Sample.MissCount, SampleMin.MissCount, SampleMax.MissCount);
	OutSample.TransmissionDistance = clamp(Sample.TransmissionDistance, SampleMin.TransmissionDistance, SampleMax.TransmissionDistance);
	return OutSample;
}

FSSDSignalSample LerpSignal(FSSDSignalSample Sample0, FSSDSignalSample Sample1, float Interp)
{
	return AddSignal(Sample0, MulSignal(SubtractSignal(Sample1, Sample0), Interp));
}

/** Normalize the signal sample to one. */
FSSDSignalSample NormalizeToOneSample(FSSDSignalSample Sample)
{
	FSSDSignalSample OutSample = MulSignal(Sample, Sample.SampleCount > 0 ? rcp(Sample.SampleCount) : 0);
	OutSample.SampleCount = Sample.SampleCount > 0 ? 1 : 0;
	return OutSample;
}

FSSDSignalSample WaveBroadcastSignal(const FWaveBroadcastSettings BroadcastSettings, FSSDSignalSample Sample)
{
	FSSDSignalSample OutSample;
	#if COMPILE_SIGNAL_COLOR
		OutSample.SceneColor = WaveBroadcast(BroadcastSettings, Sample.SceneColor);
	#endif
	#if COMPILE_SIGNAL_COLOR_ARRAY
	{
		UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
		for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
			OutSample.ColorArray[ColorId] = WaveBroadcast(BroadcastSettings, Sample.ColorArray[ColorId]);
	}
	#endif
	#if COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 2
		OutSample.ColorSH.R.V = WaveBroadcast(BroadcastSettings, Sample.ColorSH.R.V);
		OutSample.ColorSH.G.V = WaveBroadcast(BroadcastSettings, Sample.ColorSH.G.V);
		OutSample.ColorSH.B.V = WaveBroadcast(BroadcastSettings, Sample.ColorSH.B.V);
	#elif COMPILE_SIGNAL_COLOR_SH && SPHERICAL_HARMONIC_ORDER == 3
		OutSample.ColorSH.R.V0 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.R.V0);
		OutSample.ColorSH.R.V1 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.R.V1);
		OutSample.ColorSH.R.V2 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.R.V2);
		OutSample.ColorSH.G.V0 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.G.V0);
		OutSample.ColorSH.G.V1 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.G.V1);
		OutSample.ColorSH.G.V2 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.G.V2);
		OutSample.ColorSH.B.V0 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.B.V0);
		OutSample.ColorSH.B.V1 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.B.V1);
		OutSample.ColorSH.B.V2 = WaveBroadcast(BroadcastSettings, Sample.ColorSH.B.V2);
	#endif
	OutSample.SampleCount        = WaveBroadcast(BroadcastSettings, Sample.SampleCount);
	OutSample.MissCount          = WaveBroadcast(BroadcastSettings, Sample.MissCount);
	OutSample.TransmissionDistance = WaveBroadcast(BroadcastSettings, Sample.TransmissionDistance);
	return OutSample;
}

FSSDSignalFrequency MinSignalFrequency(FSSDSignalFrequency FrequencyA, FSSDSignalFrequency FrequencyB)
{
	FSSDSignalFrequency OutFrequency;
	OutFrequency.ClosestHitDistance = min(FrequencyA.ClosestHitDistance, FrequencyB.ClosestHitDistance);
	OutFrequency.WorldBluringRadius = min(FrequencyA.WorldBluringRadius, FrequencyB.WorldBluringRadius);
	OutFrequency.ConfusionFactor = min(FrequencyA.ConfusionFactor, FrequencyB.ConfusionFactor);
	return OutFrequency;
}

FSSDSignalFrequency WaveBroadcastSignalFrequency(const FWaveBroadcastSettings BroadcastSettings, FSSDSignalFrequency Frequency)
{
	FSSDSignalFrequency OutFrequency;
	OutFrequency.ClosestHitDistance = WaveBroadcast(BroadcastSettings, Frequency.ClosestHitDistance);
	OutFrequency.WorldBluringRadius = WaveBroadcast(BroadcastSettings, Frequency.WorldBluringRadius);
	OutFrequency.ConfusionFactor = WaveBroadcast(BroadcastSettings, Frequency.ConfusionFactor);
	return OutFrequency;
}
