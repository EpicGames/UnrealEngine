// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

//------------------------------------------------------- CONFIG DISABLED DEFAULTS

#ifndef CONFIG_SPECULAR_RATIO_ESTIMATOR
	#define CONFIG_SPECULAR_RATIO_ESTIMATOR 0
#endif

/** Color space to transform the color to. */
#ifndef CONFIG_COLOR_SPACE_TRANSFORMATION
	#define CONFIG_COLOR_SPACE_TRANSFORMATION COLOR_SPACE_TRANSFORMATION_DISABLED
#endif


//------------------------------------------------------- INCLUDE

#include "SSDCommon.ush"
#include "SSDSignalCore.ush"

#include "DenoisingCommon.ush"

#include "../ColorSpace.ush"
#include "../EyeAdaptationCommon.ush"
#include "../LightShaderParameters.ush"

#if !defined(CONFIG_SIGNAL_PROCESSING)
	#error Missing CONFIG_SIGNAL_PROCESSING

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
	#include "SSDPublicHarmonics.ush"

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
	
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
	#include "../MonteCarlo.ush"
	#include "../BRDF.ush"
	#include "../DeferredShadingCommon.ush"
	#include "../CommonViewUniformBuffer.ush"

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY

#else
	#error Unknown CONFIG_SIGNAL_PROCESSING

#endif


//------------------------------------------------------- DOMAIN SPECIFIC KNOWLEDGE PARAMETERS

/** Knowledge specific to the signal being. */
struct FSSDSignalDomainKnowledge
{
	// Light specific parameters
	FLightShaderParameters Light;

	// Directional light specific.
	float HitDistanceToWorldBluringRadius;

	// The type of light.
	uint LightType;
};

float4 LightPositionAndRadius[MAX_SIGNAL_BATCH_SIZE];
float4 LightDirectionAndLength[MAX_SIGNAL_BATCH_SIZE];
DECLARE_SCALAR_ARRAY(float, HitDistanceToWorldBluringRadius, MAX_SIGNAL_BATCH_SIZE);
DECLARE_SCALAR_ARRAY(uint, LightType, MAX_SIGNAL_BATCH_SIZE);

uint FrameIndex;

/** Returns the FLightShaderParameters from the root shader parameters. */
FSSDSignalDomainKnowledge GetSignalDomainKnowledge(uint BatchedSignalId)
{
	// Hopefully one day the shader compiler will be so nice we would no longer have to do this.
	FSSDSignalDomainKnowledge SignalDomain;
	SignalDomain.Light.InvRadius = 0;
	SignalDomain.Light.Color = 0;
	SignalDomain.Light.FalloffExponent = 0;
	SignalDomain.Light.Tangent = 0;
	SignalDomain.Light.SpotAngles = 0;
	SignalDomain.Light.SpecularScale = 0;
	SignalDomain.Light.DiffuseScale = 0;
	SignalDomain.Light.SoftSourceRadius = 0;	
	SignalDomain.Light.TranslatedWorldPosition = LightPositionAndRadius[BatchedSignalId].xyz;
	SignalDomain.Light.Direction = LightDirectionAndLength[BatchedSignalId].xyz;
	SignalDomain.Light.SourceRadius = LightPositionAndRadius[BatchedSignalId].w;
	SignalDomain.Light.SourceLength = LightDirectionAndLength[BatchedSignalId].w;
	SignalDomain.HitDistanceToWorldBluringRadius = GET_SCALAR_ARRAY_ELEMENT(HitDistanceToWorldBluringRadius, BatchedSignalId);
	
	#if !defined(DIM_LIGHT_TYPE)
	{
		SignalDomain.LightType = GET_SCALAR_ARRAY_ELEMENT(LightType, BatchedSignalId);
	}
	#else
	{
		SignalDomain.LightType = DIM_LIGHT_TYPE;
	}
	#endif
	return SignalDomain;
}
	


//------------------------------------------------------- INTERNAL

float Luma4(float3 Color)
{
	return (Color.g * 2.0) + (Color.r + Color.b);
}

float KarisHdrWeightY(float Color, float Exposure) 
{
	return rcp(Color * Exposure + 4.0);
}

float KarisHdrWeightInvY(float Color, float Exposure) 
{
	return 4.0 * rcp(1.0 - Color * Exposure);
}

float Luma_To_LumaLog(float x)
{
	return max(x > 0 ? log2(x) : -100, -100);
}

/** Returns the exposure multiplier that should be used for the color transformation of the signal. */
float GetSignalColorTransformationExposure()
{
	// TODO(Denoiser): expose that.
	const float Tweak = 1;

	float FrameExposureScale = EyeAdaptationLookup();
	
	FrameExposureScale *= View.OneOverPreExposure;

	return ToScalarMemory(FrameExposureScale * Tweak);
}

/** Returns the linear color space Luma with a 4 multiplier for ALU perf reasons. */
float GetSignalLuma4(FSSDSignalSample Sample, const uint Basis)
#if COMPILE_SIGNAL_COLOR
{
	// TODO(Denoiser): should force this to be scalar load.
	float FrameExposureScale = GetSignalColorTransformationExposure();
		
	// TODO(Denoiser): exposes optimisation if sample is normalized.
	const bool bIsNormalizedSample = false;

	if (Basis & COLOR_SPACE_KARIS_WEIGHTING)
	{
		float KarisX = Luma4(Sample.SceneColor.rgb);
		if ((Basis & 0x3))
		{
			KarisX = Sample.SceneColor.x;
		}

		if (!bIsNormalizedSample)
		{
			KarisX *= SafeRcp(Sample.SampleCount);
		}
		
		return KarisX * KarisHdrWeightInvY(KarisX, FrameExposureScale);
	}
	else
	{
		if ((Basis & 0x3))
		{
			return Sample.SceneColor.x;
		}

		return Luma4(Sample.SceneColor.rgb);
	}
}
#else
{
	return 0;
}
#endif

/** Conveniently transform one simple from a source basis to a destination basis. */
FSSDSignalSample TransformSignal(FSSDSignalSample Sample, const uint SrcBasis, const uint DestBasis)
#if COMPILE_SIGNAL_COLOR
{
	// If the source and destination bases are the same, early return to not pay ALU and floating point divergence.
	if (SrcBasis == DestBasis)
	{
		return Sample;
	}

	// TODO(Denoiser): should force this to be scalar load.
	float FrameExposureScale = GetSignalColorTransformationExposure();
		
	// TODO(Denoiser): exposes optimisation if sample is normalized.
	const bool bIsNormalizedSample = false;
	const bool bDebugForceNormalizeColor = true;
	const bool bIsNormalizedColor = bIsNormalizedSample || bDebugForceNormalizeColor;


	const bool bUnchangeAlphaPremultiply = (SrcBasis & COLOR_SPACE_UNPREMULTIPLY) == (DestBasis & COLOR_SPACE_UNPREMULTIPLY);
	const bool bUnchangeColorSpace = bUnchangeAlphaPremultiply && (SrcBasis & 0x3) == (DestBasis & 0x3);

	if (bDebugForceNormalizeColor)
	{
		Sample.SceneColor *= SafeRcp(Sample.SampleCount);
	}

	// Remove Karis weighting before doing any color transformations.
	if (SrcBasis & COLOR_SPACE_KARIS_WEIGHTING)
	{
		float KarisX = Luma4(Sample.SceneColor.rgb);
		if ((SrcBasis & 0x3))
		{
			KarisX = Sample.SceneColor.x;
		}

		if (!bIsNormalizedColor)
		{
			KarisX *= SafeRcp(Sample.SampleCount);
		}

		Sample.SceneColor *= KarisHdrWeightInvY(KarisX, FrameExposureScale);
	}
	
	if (bUnchangeColorSpace)
	{
		// NOP
	}
	else if ((SrcBasis & 0x3) == COLOR_SPACE_YCOCG)
	{
		Sample.SceneColor.rgb = YCoCg_2_LinearRGB(Sample.SceneColor.rgb);
	}
	else if ((SrcBasis & 0x3) == COLOR_SPACE_LCOCG)
	{
		Sample.SceneColor.rgb = LCoCg_2_LinearRGB(Sample.SceneColor.rgb);
	}

	float Alpha = Sample.SceneColor.a * SafeRcp(Sample.SampleCount);
	if (bIsNormalizedColor)
	{
		Alpha = Sample.SceneColor.a;
	}

	// Premultiplied RGBA
	if (bUnchangeAlphaPremultiply)
	{
		// NOP
	}
	else if (SrcBasis & COLOR_SPACE_UNPREMULTIPLY)
	{
		Sample.SceneColor.rgb *= Alpha;
	}
	else //if (DestBasis & COLOR_SPACE_UNPREMULTIPLY)
	{
		Sample.SceneColor.rgb *= SafeRcp(Alpha);
	}

	float x = Luma4(Sample.SceneColor.rgb);
	if ((DestBasis & 0x3) == COLOR_SPACE_YCOCG)
	{
		if (!bUnchangeColorSpace)
			Sample.SceneColor.xyz = LinearRGB_2_YCoCg(Sample.SceneColor.rgb);

		x = Sample.SceneColor.x;
	}
	else if ((DestBasis & 0x3) == COLOR_SPACE_LCOCG)
	{
		if (!bUnchangeColorSpace)
			Sample.SceneColor.xyz = LinearRGB_2_LCoCg(Sample.SceneColor.rgb);

		x = Sample.SceneColor.x;
	}

	if (DestBasis & COLOR_SPACE_KARIS_WEIGHTING)
	{
		if (bIsNormalizedColor)
		{
			Sample.SceneColor *= KarisHdrWeightY(x, FrameExposureScale);
		}
		else
		{
			Sample.SceneColor *= KarisHdrWeightY(x * SafeRcp(Sample.SampleCount), FrameExposureScale);
		}
	}
	
	if (bDebugForceNormalizeColor)
	{
		Sample.SceneColor *= Sample.SampleCount;
	}

	return Sample;
}
#else
{
	return Sample;
}
#endif

#if 0 // will probably be handy at some point.
float4 WeightedLerp( float4 ColorA, float WeightA, float4 ColorB, float WeightB, float Blend ) 
{
	float BlendA = (1.0 - Blend) * WeightA;
	float BlendB =        Blend  * WeightB;
	float RcpBlend = rcp(BlendA + BlendB);
	BlendA *= RcpBlend;
	BlendB *= RcpBlend;
	return ColorA * BlendA + ColorB * BlendB;
}
#endif


/** Create a spherical gaussian lob for a given roughness. */
FSphericalGaussian ComputeRoughnessLobe(float Roughness, float3 N, float3 V)
{
	//float a = Pow2( max( 0.02, Roughness ) );
	float a = Pow2( max( 0.001, Roughness ) );
	float a2 = a*a;
	
	float NoV = saturate( abs( dot(N, V) ) + 1e-5 );
	//float NoV = saturate( abs( dot(N, V) ) );
	
	float3 R = 2 * NoV * N - V;

	FSphericalGaussian SpecularSG;
	SpecularSG.Axis = R;
	SpecularSG.Sharpness = 0.5 / ( a2 * max( NoV, 0.1 ) );
	//SpecularSG.Sharpness = 0.5 / ( a2 * max( NoV, 0.1 ) );
	SpecularSG.Amplitude = rcp( PI * a2 );

	return SpecularSG;
}

/** Create a spherical gaussian lob for a given scene metadata. */
FSphericalGaussian ComputeRoughnessLobe(FSSDSampleSceneInfos RefSceneMetadata)
{
	#if CONFIG_USE_VIEW_SPACE
		float3 N = RefSceneMetadata.ViewNormal;
		float3 V = float3(1, 0, 0); // TODO(Denoiser).
	#else
		float3 N = RefSceneMetadata.WorldNormal;
		float3 V = -GetCameraVectorFromTranslatedWorldPosition(GetTranslatedWorldPosition(RefSceneMetadata));
	#endif

	float Roughness = RefSceneMetadata.Roughness;

	return ComputeRoughnessLobe(Roughness, N, V);
}

/** Compute the factor to apply to inifinity bluring radius based on the sample's hit distance. */
float ComputeBluringOutOfFocusOfNormalizedSample(FSSDSignalFrequency SampleFrequency, FSSDSampleSceneInfos SceneMetadata)
{
	float DistanceCameraToPixel = GetDistanceToCameraFromViewVector(View.TranslatedWorldCameraOrigin - GetTranslatedWorldPosition(SceneMetadata));

	return ComputeDenoisingCoc(DistanceCameraToPixel, SampleFrequency.ClosestHitDistance);
}

/** Project the specular lobe in screen space. Returns the major axis of the lobe anysotropy, and the inifinity bluring in ViewportUV.x space. */
void ProjectSpecularLobeToScreenSpace(
	FSSDSampleSceneInfos SceneMetadata,
	out float2 NormalizedScreenMajorAxis,
	out float InifinityMajorViewportRadius,
	out float InifinityMinorViewportRadius)
{
	float3 N = GetWorldNormal(SceneMetadata);
	float3 V = -GetCameraVectorFromTranslatedWorldPosition(GetTranslatedWorldPosition(SceneMetadata));

	// Project reflection axis onto screen.
	{
		float3 R = 2 * dot(V, N) * N - V;

		float4 RayEndClip = mul(float4(GetTranslatedWorldPosition(SceneMetadata) + R * (0.5 * SceneMetadata.WorldDepth), 1), View.TranslatedWorldToClip);
		float2 RayEndScreen = RayEndClip.xy * rcp(RayEndClip.w);

		float2 RayDirectionScreen = RayEndScreen - SceneMetadata.ScreenPosition;

		NormalizedScreenMajorAxis = normalize(RayDirectionScreen * View.ViewSizeAndInvSize.xy);
	}

	{
		float MajorRoughnessLobeAngle;
		float MinorRoughnessLobeAngle;
		ComputeSpecularLobeAngles(V, N, SceneMetadata.Roughness, /* out */ MajorRoughnessLobeAngle, /* out */ MinorRoughnessLobeAngle);
	
		/** Return the size of the bluring radius caused by the specular lobe in ViewportUV.x space. */
		InifinityMajorViewportRadius = LobeAngleToViewportRadius(MajorRoughnessLobeAngle);
		InifinityMinorViewportRadius = LobeAngleToViewportRadius(MinorRoughnessLobeAngle);
	}
}


//------------------------------------------------------- MAIN FUNCTIONS TO IMPLEMENT FOR THE PASSES

float3 ComputeMajorLightRayDirection(FSSDSampleSceneInfos SceneMetadata, FSSDSignalDomainKnowledge DomainKnowledge)
#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
{
	BRANCH
	if (DomainKnowledge.LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		return -DomainKnowledge.Light.Direction;
	}
	else
	{
		float3 PixelToLightWorldVector = DomainKnowledge.Light.TranslatedWorldPosition - GetTranslatedWorldPosition(SceneMetadata);
		return normalize(PixelToLightWorldVector);
	}
}
#else
{
	// Not used by any pass.
	return float3(0.0, 0.0, 1.0);
}
#endif

float ComputeLightSourceRadius(FSSDSignalDomainKnowledge DomainKnowledge)
#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
{
	return  DomainKnowledge.Light.SourceRadius;
}
#else
{
	return 0.0f;
}
#endif

/** COmpute the weight of the bilateral filter to use between a reference sample and a neighbor. */
float ComputeBilateralWeight(
	const uint BilateralSettings,
	float MaxWorldBluringDistance,
	FSSDSignalDomainKnowledge DomainKnowledge,
	FSSDSampleSceneInfos RefSceneMetadata,
	FSSDSampleSceneInfos NeighborSceneMetadata,
	float3 NeighborToRefVector)
{
	if (DEBUG_DISABLE_BILATERAL)
	{
		return 1;
	}
	
	#if CONFIG_USE_VIEW_SPACE
		float3 RefNormal = RefSceneMetadata.ViewNormal;
		float3 NeighborNormal = NeighborSceneMetadata.ViewNormal;
	#else
		float3 RefNormal = RefSceneMetadata.WorldNormal;
		float3 NeighborNormal = NeighborSceneMetadata.WorldNormal;
	#endif
	
	// TODO(Denoiser): extremely expensive with rcp()...
	float InvMaxWorldBluringDistance = rcp(MaxWorldBluringDistance);

	float PositionBilateralWeight = 1;
	if ((BilateralSettings & 0xF) == 1)
	{
		const float WorldRadius = MaxWorldBluringDistance;

		float RefAnisotropyInvFactor = ComputeAnisotropyInvFactor(RefSceneMetadata);

		float3 CameraToRef = View.ViewForward;

		float Z = dot(CameraToRef, NeighborToRefVector);
		float XY = length2(NeighborToRefVector - CameraToRef * Z * (1 - RefAnisotropyInvFactor));

		float DistSquare = XY;

		float Multiplier = rcp(WorldRadius * WorldRadius);

		PositionBilateralWeight = saturate(1 - DistSquare * Multiplier);
	}
	else if ((BilateralSettings & 0xF) == 2)
	{
		// TODO(Denoiser): save 2 VGPR by using SceneDepth instead of RefSceneMetadata.TranslatedWorldPosition (not sure it can work in math yet).
		// float2 ClipPosition = GetScreenPositionForProjectionType(ScreenPosition, WorldDepth);
		// Infos.TranslatedWorldPosition = mul(float4(ClipPosition, WorldDepth, 1), ScreenToTranslatedWorld).xyz;

		float DistFromRefPlane = abs(dot(RefNormal, NeighborToRefVector));

		PositionBilateralWeight = saturate(1 - DistFromRefPlane * InvMaxWorldBluringDistance);
	}
	else if ((BilateralSettings & 0xF) == 3)
	{
		if (dot(NeighborToRefVector, NeighborToRefVector) > MaxWorldBluringDistance * MaxWorldBluringDistance)
		{
			PositionBilateralWeight = 0;
		}
	}
	else if ((BilateralSettings & 0xF) == 4)
	{
		// TODO(Denoiser): extremely expensive with rcp()...
		PositionBilateralWeight = saturate(1 - dot(NeighborToRefVector, NeighborToRefVector) * rcp(MaxWorldBluringDistance * MaxWorldBluringDistance));
	}
	else if ((BilateralSettings & 0xF) == 5)
	{
		float LightTangentBilateral;
		{
			// Projet in the NeighborToRefVector into the the light tangential plane.
			float3 NeighborToLight = ComputeMajorLightRayDirection(NeighborSceneMetadata, DomainKnowledge);
			float3 ProjectedNeighborToRefVector = NeighborToRefVector - NeighborToLight * dot(NeighborToLight, NeighborToRefVector);

			// Measure in the tangential space the distance to take into account anisotropy.
			LightTangentBilateral = saturate(1 - dot(ProjectedNeighborToRefVector, ProjectedNeighborToRefVector) * (InvMaxWorldBluringDistance * InvMaxWorldBluringDistance));
		}
		
		float PlaneMisAlignementBilateral;
		{
			// Use source radius as encoding error to add more penalty to depth to avoid SSS backface transmission rim for spotlight with source radius > 0.
			// It makes the scaling of the distance by (DistanceFromLight - HitT)/HitT instead of (DistanceFromLight - HitT)/HitT*SourceRadius
			// When light is behind close to neighbor in view direction, the light tangent bilaterial will be close to 1. Little penalty
			// is applied in depth. Adding the effect of SourceRadius on NeighborToRefError could add more penalty to depth to mitigate the
			// bleeding error.
			const bool bApplyDepthPenaltyWeightForSSS = RefSceneMetadata.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE && ComputeLightSourceRadius(DomainKnowledge) > 0.0f;
			const float	NormalEncodingError = bApplyDepthPenaltyWeightForSSS ? DomainKnowledge.Light.SourceRadius : -0.25;

			
			float DistFromRefPlane = dot(RefNormal, -NeighborToRefVector);

			float NeighborToRefPlaneDistance = length(-NeighborToRefVector - DistFromRefPlane * RefNormal);
			float NeighborToRefError = NeighborToRefPlaneDistance * NormalEncodingError;

			PlaneMisAlignementBilateral = saturate(1 - 6 * (abs(DistFromRefPlane) + NeighborToRefError) * InvMaxWorldBluringDistance);
		}

		PositionBilateralWeight = LightTangentBilateral * PlaneMisAlignementBilateral;
	}
	else if ((BilateralSettings & 0xF) == 6)
	{
		float PlaneMisAlignementBilateral;
		{
			const float NormalEncodingError = View.GeneralPurposeTweak;
			
			float DistFromRefPlane = dot(RefNormal, -NeighborToRefVector);

			float NeighborToRefPlaneDistance = length(-NeighborToRefVector - DistFromRefPlane * RefNormal);
			float NeighborToRefError = NeighborToRefPlaneDistance * NormalEncodingError;

			PlaneMisAlignementBilateral = saturate(1 - (abs(DistFromRefPlane) - NeighborToRefError) * InvMaxWorldBluringDistance);
		}

		PositionBilateralWeight = PlaneMisAlignementBilateral;
	}

	float NormalBilateralWeight = 1;
	if (BilateralSettings & BILATERAL_NORMAL)
	{
		float NoN = max(dot(RefNormal, NeighborNormal), 0);

		#if 1 // TODO(Denoiser)
			NormalBilateralWeight = pow(NoN, 4);
		#else
			// [ Kontkanen et al 2004, "Irradiance Filtering for Monte Carlo Ray Tracing" ]
			// TODO(Denoiser): looks broken
			float m = MaxWorldBluringDistance;
			float a = 0.3;
			float d = sqrt(1 - NoN); // + length(RefSceneMetadata.TranslatedWorldPosition - NeighborSceneMetadata.TranslatedWorldPosition) * rcp(m);
			float x = d * rcp(a);

			NormalBilateralWeight = exp(-x * x);
		#endif
	}
	
	// [ Tokoyashi 2015, "Specular Lobe-Aware Filtering and Upsampling for Interactive Indirect Illumination" ]
	float LobeSimilarity = 1;
	float AxesSimilarity = 1;
	if (BilateralSettings & (BILATERAL_TOKOYASHI_LOBE | BILATERAL_TOKOYASHI_AXES))
	{
		const float Beta = 32;

		FSphericalGaussian Ref = ComputeRoughnessLobe(RefSceneMetadata);
		FSphericalGaussian Neighbor = ComputeRoughnessLobe(NeighborSceneMetadata);
	
		float InvSharpnessSum = rcp(Ref.Sharpness + Neighbor.Sharpness);

		if (BilateralSettings & BILATERAL_TOKOYASHI_LOBE)
			LobeSimilarity = pow(2 * sqrt(Ref.Sharpness * Neighbor.Sharpness) * InvSharpnessSum, Beta);
		
		if (BilateralSettings & BILATERAL_TOKOYASHI_AXES)
			AxesSimilarity = exp(-(Beta * (Ref.Sharpness * Neighbor.Sharpness) * InvSharpnessSum) * saturate(1 - dot(Ref.Axis, Neighbor.Axis)));
	}

	if (BilateralSettings & BILATERAL_SHADING_MODEL)
	{
		FLATTEN
		if (RefSceneMetadata.ShadingModelID != NeighborSceneMetadata.ShadingModelID)
			return 0.0;
	}

	return PositionBilateralWeight * NormalBilateralWeight * LobeSimilarity * AxesSimilarity;
}

float GetSignalWorldBluringRadius(FSSDSignalFrequency SampleFrequency, FSSDSampleSceneInfos SceneMetadata, FSSDSignalDomainKnowledge DomainKnowledge)
#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
{
	if (SampleFrequency.ClosestHitDistance == DENOISER_INVALID_HIT_DISTANCE)
	{
		return WORLD_RADIUS_INVALID;
	}
	else if (SampleFrequency.ClosestHitDistance == DENOISER_MISS_HIT_DISTANCE)
	{
		return WORLD_RADIUS_MISS; // 100000;
	}

	// Sometime, artists might put occluder very close to the area light compared to they area, that may lead to negative values.
	// TODO(Denoiser): the correct way to fix this is to move this world bluring radius computation into the RGS, and have DistanceFromLight = Ray's MaxT.
	const float ClosestLightDistance = 0.001;

	BRANCH
	if (DomainKnowledge.LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		return DomainKnowledge.HitDistanceToWorldBluringRadius * SampleFrequency.ClosestHitDistance;
	}
	else
	{
		return ComputeLightSampleWorldBluringRadius(
			GetTranslatedWorldPosition(SceneMetadata),
			DomainKnowledge.LightType, DomainKnowledge.Light,
			SampleFrequency.ClosestHitDistance);
	}
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
{
	// No need to compute the world bluring radius given the hit has been classified by harmonic frequencies already.
	return 0;
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
{
	// Not used by any pass.
	return 0;
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
{
	if (SampleFrequency.ClosestHitDistance == DENOISER_INVALID_HIT_DISTANCE)
	{
		return WORLD_RADIUS_INVALID;
	}
	else if (SampleFrequency.ClosestHitDistance == DENOISER_MISS_HIT_DISTANCE)
	{
		return WORLD_RADIUS_MISS; // 100000;
	}

	// How much can be blured is about the the length of hit distance assuming some amount of visiiblity.
	return SampleFrequency.ClosestHitDistance;
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC
{
	// Not used by any pass.
	return 0;
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
{
	// Not used by any pass.
	return 0;
}
#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
{
	// Not used by any pass.
	return 0;
}
#else
	#error Unimplemented GetSignalWorldBluringRadius()
#endif

// Returns the penumbra of this sample, or 1 if invalid.
float GetSamplePenumbraSafe(FSSDSignalSample Sample)
{
	return (Sample.SampleCount > 0 ? Sample.MissCount / Sample.SampleCount : 1);
}

// Get penumbra, but be aware of NaN.
float GetSamplePenumbra(FSSDSignalSample Sample)
{
	if (1) // TODO(Denoiser): For debugging purpose of division by 0.
	{
		return GetSamplePenumbraSafe(Sample);
	}
	return Sample.MissCount / Sample.SampleCount;
}


// Matches GenerateCosineNormalRay()
// TODO(Denoiser): expose a Weight that multiply directly onto the Sample.SceneColor to reduce ALU
#if COMPILE_SIGNAL_COLOR_SH && COMPILE_SIGNAL_COLOR
FSSDSphericalHarmonicRGB ComputeSampleColorSH(FSSDSampleSceneInfos SampleSceneMetadata, FSSDSignalSample Sample, uint2 NeighborPixelCoord)
{
	float2 RandSample = RandomSequenceCreate(uint3(NeighborPixelCoord, FrameIndex), 0, 1).Get2D();

	float4 TangentDirection = CosineSampleHemisphere(RandSample);
	float3 WorldDirection = TangentToWorld(TangentDirection.xyz, GetWorldNormal(SampleSceneMetadata));

	#if SPHERICAL_HARMONIC_ORDER == 2
		FSSDSphericalHarmonic SampleBasis = SHBasisFunction(WorldDirection);
		return MulSH(SampleBasis, Sample.SceneColor.rgb);

	#elif SPHERICAL_HARMONIC_ORDER == 3
		FSSDSphericalHarmonic SampleBasis = SHBasisFunction3(WorldDirection);
		return MulSH3(SampleBasis, Sample.SceneColor.rgb);

	#endif
}
#endif
