// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDDefinitions.ush"


//------------------------------------------------------- ENUM VALUES

/** Different possible stage for spatial accumulation. Matches  */
	#define STAGE_RECONSTRUCTION 0
	#define STAGE_PRE_CONVOLUTION 1
	#define STAGE_REJECTION_PRE_CONVOLUTION 2
	#define STAGE_POST_FILTERING 3
	#define STAGE_FINAL_OUTPUT 4

/** Policy to use to change the size of kernel. */
	#define SAMPLE_COUNT_POLICY_DISABLED 0
	#define SAMPLE_COUNT_POLICY_SAMPLE_ACCUMULATION_BASED 4

/** What signal should be outputed. */
	// Only output the sum of the signal 0.
	#define OUTPUT_MODE_SUM 0
	// Only output the sum of the momment 1 & 2 of the signal 0.
	#define OUTPUT_MODE_2MOMMENT_SUM 1

	// Output the result of descending ring bucketing.
	#define OUTPUT_MODE_DRB 2


//------------------------------------------------------- CONFIGS

#define TILE_PIXEL_SIZE 8

#define CONFIG_SIGNAL_PROCESSING DIM_SIGNAL_PROCESSING
#define CONFIG_UPSCALE           DIM_UPSCALE
#define CONFIG_SIGNAL_BATCH_SIZE DIM_SIGNAL_BATCH_SIZE
#define CONFIG_DENOISE_TRANSMISSION DIM_DENOISE_TRANSMISSION

// Configures all the pass for each individual signals.
#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
	#define MAX_SIGNAL_BATCH_SIZE CONFIG_SIGNAL_BATCH_SIZE
	#define SIGNAL_ARRAY_SIZE CONFIG_SIGNAL_BATCH_SIZE

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_MONOCHROMATIC_PENUMBRA
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_PENUMBRA_INJESTION_NSPP
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY

		#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_UINT2
		#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
		#define CONFIG_INPUT_TEXTURE_COUNT ((CONFIG_SIGNAL_BATCH_SIZE + 1) / 2)
		#define CONFIG_OUTPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS
		#define CONFIG_MAX_WITH_REF_DISTANCE 1
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB

		#if DIM_SIGNAL_BATCH_SIZE > 1
			#define CONFIG_CLAMP_UV_PER_SIGNAL 1
		#endif

	#elif DIM_STAGE == STAGE_PRE_CONVOLUTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY

		#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
		#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4

		#define CONFIG_INPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE
		#define CONFIG_OUTPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_HEXAWEB
		#define CONFIG_CUSTOM_SPREAD_FACTOR 1

		#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS
		#define CONFIG_MAX_WITH_REF_DISTANCE 1
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB

	#elif DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION
		#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
		#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4

	#elif DIM_STAGE == STAGE_POST_FILTERING
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY

		#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
		#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4

		#define CONFIG_INPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE
		#define CONFIG_OUTPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		//#define CONFIG_SAMPLE_SUBSET        1
		#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB

	#elif DIM_STAGE == STAGE_FINAL_OUTPUT
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY

		#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
		#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4

		#define CONFIG_INPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE
		#define CONFIG_OUTPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_1X1

	#else
		#error Unexpected stage.
	#endif

	// Compress the DRB accumulator to have lower VGPR footprint.
	#if defined(CONFIG_OUTPUT_MODE) && CONFIG_OUTPUT_MODE == OUTPUT_MODE_DRB
		// Looks like shader compilers completly give up.
		// #define CONFIG_ACCUMULATOR_VGPR_COMPRESSION ACCUMULATOR_COMPRESSION_PENUMBRA_DRB
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
	// Denoise diffuse and specular harmonics at the same time.
	#define MAX_SIGNAL_BATCH_SIZE 2
	#define SIGNAL_ARRAY_SIZE 2
	#undef CONFIG_SIGNAL_BATCH_SIZE
	#define CONFIG_SIGNAL_BATCH_SIZE 2

	// Each harmonic requires input and output RGB.
	#define COMPILE_SIGNAL_COLOR_ARRAY 2

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_POLYCHROMATIC_PENUMBRA
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	// Any world distance depends on the harmonic being processed.
	#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_HARMONIC

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_INPUT
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HARMONIC_RECONSTRUCTION

		#define CONFIG_INPUT_TEXTURE_COUNT 4
		#define CONFIG_OUTPUT_TEXTURE_COUNT 4

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		//#define CONFIG_MAX_WITH_REF_DISTANCE 1

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
	// Denoise only specular.
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR 1
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_REFLECTIONS

	#if DIM_STAGE == STAGE_RECONSTRUCTION || DIM_STAGE == STAGE_PRE_CONVOLUTION
		#define SIGNAL_ARRAY_SIZE 1

		// Input and output layout.
		#if DIM_STAGE == STAGE_RECONSTRUCTION 
			#define CONFIG_INPUT_TEXTURE_COUNT 2
			#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_REFLECTIONS_INPUT
		#else
			#define CONFIG_INPUT_TEXTURE_COUNT 2
			#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY
		#endif
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY
		#define CONFIG_OUTPUT_TEXTURE_COUNT 2

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_DIRECTIONAL_ELLIPSE

		// Do color accumulation with karis weighting to avoid flickering specular highlight to show up the kernel pattern.
		// TODO(Denoiser): This is a bit agressive.
		#define CONFIG_ACCUMULATION_COLOR_SPACE (COLOR_SPACE_RGB | COLOR_SPACE_KARIS_WEIGHTING)

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO
	// Denoise only AO.
	#define MAX_SIGNAL_BATCH_SIZE 1

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		#define SIGNAL_ARRAY_SIZE 1

		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_AO_INPUT
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_AO_HISTORY

		#define CONFIG_INPUT_TEXTURE_COUNT 2
		#define CONFIG_OUTPUT_TEXTURE_COUNT 1

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB

		#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_AO

	#elif DIM_STAGE == STAGE_PRE_CONVOLUTION
		#define SIGNAL_ARRAY_SIZE 1 // first and second momment to measure variance in temporal accumulation.

		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_AO_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_AO_HISTORY

		#define CONFIG_INPUT_TEXTURE_COUNT 1
		#define CONFIG_OUTPUT_TEXTURE_COUNT 1

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_HEXAWEB
		#define CONFIG_CUSTOM_SPREAD_FACTOR 1

		//#define CONFIG_MAX_WITH_REF_DISTANCE 1
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB

		#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_AO

	#elif DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION
		#define SIGNAL_ARRAY_SIZE 2 // first and second momment to measure variance in temporal accumulation.

		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_AO_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_AO_REJECTION

		#define CONFIG_INPUT_TEXTURE_COUNT 1
		#define CONFIG_OUTPUT_TEXTURE_COUNT 1

		#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_AO_HISTORY

	#elif DIM_STAGE == STAGE_POST_FILTERING
		#define SIGNAL_ARRAY_SIZE 1

		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_AO_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_AO_HISTORY

		#define CONFIG_INPUT_TEXTURE_COUNT 1
		#define CONFIG_OUTPUT_TEXTURE_COUNT 1

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_HEXAWEB
		#define CONFIG_OUTPUT_MODE          OUTPUT_MODE_DRB
		#define CONFIG_CUSTOM_SPREAD_FACTOR 1

		#define CONFIG_SAMPLE_COUNT_POLICY  SAMPLE_COUNT_POLICY_SAMPLE_ACCUMULATION_BASED

		#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_AO

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
	// Denoise diffuse and AO, but AO is FSSDSignalSample::MissCount.
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR 1

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_OUTPUT_TEXTURE_COUNT 2

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_INPUT_NSPP
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS

		#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_HIT_DISTANCE

	#elif DIM_STAGE == STAGE_PRE_CONVOLUTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_HEXAWEB
		#define CONFIG_CUSTOM_SPREAD_FACTOR 1
		//#define CONFIG_MAX_WITH_REF_DISTANCE 1

	#elif DIM_STAGE == STAGE_POST_FILTERING
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		#define CONFIG_SAMPLE_SUBSET        1

		#define CONFIG_SAMPLE_COUNT_POLICY  SAMPLE_COUNT_POLICY_SAMPLE_ACCUMULATION_BASED

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC && 0
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR_SH 1

	// Given it's a spherical harmonic that store directionality, only need position based rejection.
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC

	#define CONFIG_INPUT_TEXTURE_COUNT 4
	#define CONFIG_OUTPUT_TEXTURE_COUNT 4

	#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_MIN_METADATA

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC
	#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2
	#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC
	#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2

	// Spherical harmonics are a lot of data, need to shrink VGPR pressure to improve lattency hidding when fetching the buffer.
	// TODO(Denoiser): some shader compiler completly falls apparts with the current implementation of
	//       CONFIG_SIGNAL_VGPR_COMPRESSION and actually drops in occupency.
	#define CONFIG_SIGNAL_VGPR_COMPRESSION SIGNAL_COMPRESSION_DIFFUSE_INDIRECT_HARMONIC

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS

	#elif DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC && 1
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR_SH 1

	// Given it's a spherical harmonic that store directionality, only need position based rejection.
	#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_MIN_METADATA
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC
	//#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DISABLED
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 6.0

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_OUTPUT_TEXTURE_COUNT 2

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_INPUT
	#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		//#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		#define CONFIG_SAMPLE_SET           SAMPLE_SET_RAW_EXPERIMENTAL_KERNEL

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
	// Denoise diffuse and AO, but AO is FSSDSignalSample::MissCount.
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR 1

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE
	//#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DISABLED
	
	// SSGI doesn't have any bilateral distance computed from hitT, so allow to blur spatially by about the size of the kernel.
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 3.0

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_OUTPUT_TEXTURE_COUNT 2

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_SSGI_INPUT
		//#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10

		#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		//#define CONFIG_SAMPLE_SET           SAMPLE_SET_1X1

	#else
		#error Unexpected stage.
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
	// Denoise diffuse and AO, but AO is FSSDSignalSample::MissCount.
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	#define COMPILE_SIGNAL_COLOR_ARRAY 2

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_PROBE_HIERARCHY
	//#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DISABLED
	
	// SSGI doesn't have any bilateral distance computed from hitT, so allow to blur spatially by about the size of the kernel.
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 3.0

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_OUTPUT_TEXTURE_COUNT 2

	#if DIM_STAGE == STAGE_RECONSTRUCTION
		// Input and output layout.
		#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_INPUT
		//#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2
		#define CONFIG_SIGNAL_OUTPUT_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_HISTORY

		//#define CONFIG_SAMPLE_SET           SAMPLE_SET_STACKOWIAK_4_SETS
		#define CONFIG_SAMPLE_SET           SAMPLE_SET_1X1

	#else
		#error Unexpected stage.
	#endif

#else
	#error Unknown signal processing.
#endif

// Configures pass regardless of the signals.
#if DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION
	#define CONFIG_SAMPLE_SET           SAMPLE_SET_3X3_PLUS

	// Normalize the input, because want to measure the spatial variance regardless of how many samples where used to reconstruct the signal.
	#define CONFIG_NORMALIZE_INPUT 1

	// Output the 2 momment because history rejection is varaiance based, and may flicker with momment 2 loss since the pre
	// convolution will reduce the variance of momment 1.
	#define CONFIG_OUTPUT_MODE OUTPUT_MODE_2MOMMENT_SUM

#endif


// No previous frame reprojection, save VGPR.
//#define CONFIG_NEIGHBOR_TO_REF_COMPUTATION NEIGHBOR_TO_REF_LOWEST_VGPR_PRESSURE


//------------------------------------------------------- CONFIG DISABLED DEFAULTS

/** Whether should clamp the UV individually per texture. */
#ifndef CONFIG_CLAMP_UV_PER_SIGNAL
	#define CONFIG_CLAMP_UV_PER_SIGNAL 0
#endif

/** Changes the logic controling the number of sample to do. */
#ifndef CONFIG_SAMPLE_COUNT_POLICY
	#define CONFIG_SAMPLE_COUNT_POLICY SAMPLE_COUNT_POLICY_DISABLED
#endif

/** Selects a subset of sample of a given CONFIG_SAMPLE_SET */
#ifndef CONFIG_SAMPLE_SUBSET
	#define CONFIG_SAMPLE_SUBSET 0
#endif

/** Whether the ray tracing input may needs to be upscale to the view's resolution. */
#ifndef CONFIG_UPSCALE
	#define CONFIG_UPSCALE 0
#endif

/** Color space of the input signal. */
#ifndef CONFIG_INPUT_COLOR_SPACE
	#define CONFIG_INPUT_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif

/** Color space to use for the accumulation. */
#ifndef CONFIG_ACCUMULATION_COLOR_SPACE
	#define CONFIG_ACCUMULATION_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif

/** Color space to output in the signal. */
#ifndef CONFIG_OUTPUT_COLOR_SPACE
	#define CONFIG_OUTPUT_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif

/** Removes the highest color. */
#ifndef CONFIG_REJECT_HIGHEST_COLOR
	#define CONFIG_REJECT_HIGHEST_COLOR 0
#endif

/** Whether the input signal should be normalized. */
#ifndef CONFIG_NORMALIZE_INPUT
	#define CONFIG_NORMALIZE_INPUT 0
#endif

/** The oupput mode that should be use. */
#ifndef CONFIG_OUTPUT_MODE
	#define CONFIG_OUTPUT_MODE OUTPUT_MODE_SUM
#endif

/** The number of signal that should be processed per signal domain. */
#ifndef CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN SIGNAL_ARRAY_SIZE
#endif

/** Selects how the world distance should be computed for bilateral rejection. */
#ifndef CONFIG_BILATERAL_DISTANCE_COMPUTATION
	#define CONFIG_BILATERAL_DISTANCE_COMPUTATION SIGNAL_WORLD_FREQUENCY_MIN_METADATA
#endif

/** Adds a multiplier on how the distance should be computed. */
#ifndef CONFIG_BILATERAL_DISTANCE_MULTIPLIER
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 1.0
#endif

/** Whether neighbor bilateral distance should be maxed with reference one. */
#ifndef CONFIG_MAX_WITH_REF_DISTANCE
	#define CONFIG_MAX_WITH_REF_DISTANCE 0
#endif


//------------------------------------------------------- COMPILATION CONFIGURATION

// Choose kernel to compile.
#if CONFIG_SAMPLE_SET == SAMPLE_SET_STACKOWIAK_4_SETS
	#define COMPILE_STACKOWIAK_KERNEL 1
#elif CONFIG_SAMPLE_SET == SAMPLE_SET_HEXAWEB
	#define COMPILE_DISK_KERNEL 1
#elif CONFIG_SAMPLE_SET == SAMPLE_SET_DIRECTIONAL_RECT || CONFIG_SAMPLE_SET == SAMPLE_SET_DIRECTIONAL_ELLIPSE
	#define COMPILE_DIRECTIONAL_KERNEL 1
#elif CONFIG_SAMPLE_SET == SAMPLE_SET_RAW_EXPERIMENTAL_KERNEL
	#define COMPILE_RAW_EXPERIMENTAL_KERNEL 1
#else
	#define COMPILE_BOX_KERNEL 1
#endif

// Choose accumulators to compile.
#if CONFIG_OUTPUT_MODE == OUTPUT_MODE_DRB
	#define COMPILE_DRB_ACCUMULATOR 1
	#define COMPILE_MIN_FREQUENCY_ACCUMULATOR 1

#elif CONFIG_OUTPUT_MODE == OUTPUT_MODE_2MOMMENT_SUM
	#define COMPILE_MOMENT1_ACCUMULATOR 1
	#define COMPILE_MOMENT2_ACCUMULATOR 1

#elif CONFIG_OUTPUT_MODE == OUTPUT_MODE_SUM
	#define COMPILE_MOMENT1_ACCUMULATOR 1
	#define COMPILE_MIN_FREQUENCY_ACCUMULATOR 1

#else
	#error Unknown output mode.
#endif

//------------------------------------------------------- INCLUDES

#include "SSDSignalFramework.ush"
#include "SSDSignalArray.ush"
#include "SSDSpatialKernel.ush"


//------------------------------------------------------- LATE CONFIG DEFAULTS

/** Choose how the reference metadata should be compressed. */
#ifndef CONFIG_REF_METADATA_COMPRESSION
	#define CONFIG_REF_METADATA_COMPRESSION CONFIG_METADATA_BUFFER_LAYOUT
#endif


//------------------------------------------------------- PARAMETERS

uint MaxSampleCount;
uint PreviousCumulativeMaxSampleCount;
uint UpscaleFactor;

#if !CONFIG_UPSCALE && CONFIG_CUSTOM_SPREAD_FACTOR
	float KernelSpreadFactor;
#endif

float HarmonicPeriode;

float4 InputBufferUVMinMax[CONFIG_SIGNAL_BATCH_SIZE];


#if !defined(CONFIG_INPUT_TEXTURE_COUNT)
	#error Missing CONFIG_INPUT_TEXTURE_COUNT
#endif

FSSDTexture2D SignalInput_Textures_0;
FSSDTexture2D SignalInputUint_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
FSSDTexture2D SignalInput_Textures_1;
FSSDTexture2D SignalInputUint_Textures_1;
#else
#define SignalInput_Textures_1 SignalInput_Textures_0
#define SignalInputUint_Textures_1 SignalInputUint_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
FSSDTexture2D SignalInput_Textures_2;
FSSDTexture2D SignalInputUint_Textures_2;
#else
#define SignalInput_Textures_2 SignalInput_Textures_0
#define SignalInputUint_Textures_2 SignalInputUint_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
FSSDTexture2D SignalInput_Textures_3;
FSSDTexture2D SignalInputUint_Textures_3;
#else
#define SignalInput_Textures_3 SignalInput_Textures_0
#define SignalInputUint_Textures_3 SignalInputUint_Textures_0
#endif

#if DIM_STAGE == STAGE_FINAL_OUTPUT && \
    CONFIG_SIGNAL_PROCESS == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK && \
    !CONFIG_DENOISE_TRANSMISSION

Texture2D OriginalInput_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
Texture2D OriginalInput_Textures_1;
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
Texture2D OriginalInput_Textures_2;
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
Texture2D OriginalInput_Textures_3;
#endif
#endif

#if !defined(CONFIG_OUTPUT_TEXTURE_COUNT)
	#error Missing CONFIG_OUTPUT_TEXTURE_COUNT
#endif

FSSDRWTexture2D	SignalOutput_UAVs_0;

#if CONFIG_OUTPUT_TEXTURE_COUNT > 1
FSSDRWTexture2D	SignalOutput_UAVs_1;
#else
#define SignalOutput_UAVs_1 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 2
FSSDRWTexture2D	SignalOutput_UAVs_2;
#else
#define SignalOutput_UAVs_2 SignalOutput_UAVs_0
#endif

#if CONFIG_OUTPUT_TEXTURE_COUNT > 3
FSSDRWTexture2D	SignalOutput_UAVs_3;
#else
#define SignalOutput_UAVs_3 SignalOutput_UAVs_0
#endif


//------------------------------------------------------- FUNCTIONS

// TODO(Denoiser): duplicated with reflection code.
uint2 GetPixelCoord(uint2 DispatchThreadId)
{
	uint UpscaleFactorPow2 = UpscaleFactor * UpscaleFactor;

	// TODO(Denoiser): find a way to not interfer with TAA's jittering.
	uint SubPixelId = View.StateFrameIndex & (UpscaleFactorPow2 - 1);

	return DispatchThreadId * UpscaleFactor + uint2(SubPixelId & (UpscaleFactor - 1), SubPixelId / UpscaleFactor);
}


//------------------------------------------------------- ENTRY POINTS

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
#if CONFIG_SIGNAL_INPUT_TEXTURE_TYPE == SIGNAL_TEXTURE_TYPE_FLOAT4
	Texture2D Signal_Textures_0 = SignalInput_Textures_0;
	Texture2D Signal_Textures_1 = SignalInput_Textures_1;
	Texture2D Signal_Textures_2 = SignalInput_Textures_2;
	Texture2D Signal_Textures_3 = SignalInput_Textures_3;
#else
	FSSDTexture2D Signal_Textures_0 = SignalInput_Textures_0;
	FSSDTexture2D Signal_Textures_1 = SignalInput_Textures_1;
	FSSDTexture2D Signal_Textures_2 = SignalInput_Textures_2;
	FSSDTexture2D Signal_Textures_3 = SignalInput_Textures_3;
#endif

	// Find out scene buffer UV.
	float2 SceneBufferUV = DispatchThreadId * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;
	if (true)
	{
		SceneBufferUV = clamp(SceneBufferUV, DenoiserBufferBilinearUVMinMax.xy, DenoiserBufferBilinearUVMinMax.zw);
	}
	
	// Read reference meta data.
	FSSDCompressedSceneInfos CompressedRefSceneMetadata;
	FSSDSampleSceneInfos RefSceneMetadata;
	{
		CompressedRefSceneMetadata = SampleCompressedSceneMetadata(
			/* bPrevFrame = */ false,
			SceneBufferUV, BufferUVToBufferPixelCoord(SceneBufferUV));
		
		float2 ScreenPosition = DenoiserBufferUVToScreenPosition(SceneBufferUV);
	
		RefSceneMetadata = UncompressSampleSceneInfo(
			CONFIG_METADATA_BUFFER_LAYOUT, /* bPrevFrame = */ false,
			ScreenPosition, CompressedRefSceneMetadata);
	}

	// Sample the reference sample.
	#if !CONFIG_UPSCALE || 1
		FSSDSignalArray RefSamples;
		FSSDSignalFrequencyArray RefFrequencies;
		SampleMultiplexedSignals(
			Signal_Textures_0,
			Signal_Textures_1,
			Signal_Textures_2,
			Signal_Textures_3,
			GlobalPointClampedSampler,
			CONFIG_SIGNAL_INPUT_LAYOUT,
			/* MultiplexedSampleId = */ 0,
			/* bNormalizeSample = */ CONFIG_NORMALIZE_INPUT != 0,
			SceneBufferUV,
			/* out */ RefSamples,
			/* out */ RefFrequencies);
		
		#if CONFIG_NORMALIZE_INPUT
			FSSDSignalArray NormalizedRefSamples = RefSamples;
		#else
			// TODO(Denoiser): Decode twice instead.
			FSSDSignalArray NormalizedRefSamples = NormalizeToOneSampleArray(RefSamples);
		#endif
	#endif

	//DebugOutput[DispatchThreadId] = float4(GetWorldNormal(RefSceneMetadata)* 0.5 + 0.5, GetWorldDepth(RefSceneMetadata));

	/** factor by witch should be spread out. */
	#if CONFIG_UPSCALE
		float KernelSpreadFactor = UpscaleFactor;
	#elif !CONFIG_CUSTOM_SPREAD_FACTOR
		const float KernelSpreadFactor = 1;
	#endif

	/** Find out the number of samples that should be done. */
	float RequestedSampleCount = 1024;
	
	#if CONFIG_SAMPLE_SET == SAMPLE_SET_NONE
		RequestedSampleCount = 1;
	#elif CONFIG_SAMPLE_COUNT_POLICY == SAMPLE_COUNT_POLICY_DISABLED
		// NOP
	#elif CONFIG_SAMPLE_COUNT_POLICY == SAMPLE_COUNT_POLICY_SAMPLE_ACCUMULATION_BASED
	{
		#if CONFIG_SIGNAL_BATCH_SIZE != 1
			#error Unable to support more than one signal.
		#endif
		RequestedSampleCount = clamp(TARGETED_SAMPLE_COUNT / RefSamples.Array[0].SampleCount, 1, MaxSampleCount);
	}
	#else
		#error Unknown policy to control the number of samples.
	#endif

	// Register renaming of members of FSSDKernelConfig to survive until the output to UAV
	#if (CONFIG_SAMPLE_SET == SAMPLE_SET_STACKOWIAK_4_SETS) && CONFIG_VGPR_OPTIMIZATION
		float2 KernelBufferUV;
		uint SampleTrackId;
	#endif

	// Accumulate spatially the input.
	FSSDSignalAccumulatorArray SignalAccumulators;
	{
		FSSDKernelConfig KernelConfig = CreateKernelConfig();

		#if DEBUG_OUTPUT
		{
			KernelConfig.DebugPixelPosition = DispatchThreadId;
			KernelConfig.DebugEventCounter = 0;
		}
		#endif

		// Compile time.
		KernelConfig.SampleSet = CONFIG_SAMPLE_SET;
		KernelConfig.SampleSubSetId = CONFIG_SAMPLE_SUBSET;
		KernelConfig.BufferLayout = CONFIG_SIGNAL_INPUT_LAYOUT;
		KernelConfig.MultiplexedSignalsPerSignalDomain = CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
		KernelConfig.NeighborToRefComputation = NEIGHBOR_TO_REF_LOWEST_VGPR_PRESSURE;
		KernelConfig.bUnroll = CONFIG_SAMPLE_SET != SAMPLE_SET_STACKOWIAK_4_SETS;
		KernelConfig.bDescOrder = CONFIG_OUTPUT_MODE == OUTPUT_MODE_DRB;
		KernelConfig.BilateralDistanceComputation = CONFIG_BILATERAL_DISTANCE_COMPUTATION;
		KernelConfig.WorldBluringDistanceMultiplier = CONFIG_BILATERAL_DISTANCE_MULTIPLIER;
		KernelConfig.bNormalizeSample = CONFIG_NORMALIZE_INPUT != 0;
		KernelConfig.bSampleKernelCenter = CONFIG_UPSCALE;
		KernelConfig.bForceKernelCenterAccumulation = true;
		KernelConfig.bClampUVPerMultiplexedSignal = CONFIG_CLAMP_UV_PER_SIGNAL != 0;

		// Reconstruct the spherical harmonic when reconstructing from 1spp.
		KernelConfig.bComputeSampleColorSH = DIM_STAGE == STAGE_RECONSTRUCTION && DIM_MULTI_SPP == 0;
		
		{
			UNROLL_N(SIGNAL_ARRAY_SIZE)
			for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
			{
				KernelConfig.BufferColorSpace[MultiplexId] = CONFIG_INPUT_COLOR_SPACE;
				KernelConfig.AccumulatorColorSpace[MultiplexId] = CONFIG_ACCUMULATION_COLOR_SPACE;
			}
		}

		SetBilateralPreset(CONFIG_BILATERAL_PRESET, /* inout */ KernelConfig);

		// SGPRs
		KernelConfig.BufferSizeAndInvSize = DenoiserBufferSizeAndInvSize;
		KernelConfig.BufferBilinearUVMinMax = DenoiserBufferBilinearUVMinMax;
		KernelConfig.KernelSpreadFactor = KernelSpreadFactor;
		KernelConfig.HarmonicPeriode = HarmonicPeriode;

		#if CONFIG_CLAMP_UV_PER_SIGNAL
		{
			UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
			for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
			{
				uint MultiplexId = BatchedSignalId / CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
				KernelConfig.PerSignalUVMinMax[MultiplexId] = InputBufferUVMinMax[MultiplexId];
			}
		}
		#endif

		// VGPRs
		KernelConfig.BufferUV = SceneBufferUV; 
		{
			#if CONFIG_REF_METADATA_COMPRESSION == CONFIG_METADATA_BUFFER_LAYOUT
				// Straight up plumb down the compress layout to save any VALU.
				KernelConfig.CompressedRefSceneMetadata = CompressedRefSceneMetadata;
			#else
				// Recompress the reference scene metadata
				KernelConfig.CompressedRefSceneMetadata = CompressSampleSceneInfo(CONFIG_REF_METADATA_COMPRESSION, RefSceneMetadata);
			#endif
			KernelConfig.RefBufferUV = SceneBufferUV;
			KernelConfig.RefSceneMetadataLayout = CONFIG_REF_METADATA_COMPRESSION;
		}
		KernelConfig.HammersleySeed = Rand3DPCG16(int3(SceneBufferUV * BufferUVToOutputPixelPosition, View.StateFrameIndexMod8)).xy;

		// Set up reference distance for all signals.
		#if CONFIG_MAX_WITH_REF_DISTANCE
		{
			KernelConfig.bMaxWithRefBilateralDistance = true;

			UNROLL_N(SIGNAL_ARRAY_SIZE)
			for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
			{
				if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS)
				{
					KernelConfig.RefBilateralDistance[MultiplexId] = RefFrequencies.Array[MultiplexId].WorldBluringRadius;
				}
				else if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_HIT_DISTANCE)
				{
					KernelConfig.RefBilateralDistance[MultiplexId] = RefFrequencies.Array[MultiplexId].ClosestHitDistance;
				}
				else
				{
					const uint BatchedSignalId = ComputeSignalBatchIdFromSignalMultiplexId(KernelConfig, MultiplexId);
					FSSDSignalDomainKnowledge DomainKnowledge = GetSignalDomainKnowledge(BatchedSignalId);

					KernelConfig.RefBilateralDistance[MultiplexId] = GetSignalWorldBluringRadius(RefFrequencies.Array[MultiplexId], RefSceneMetadata, DomainKnowledge);
				}
			}
		}
		#endif

		// When doing history rejection preconvolution may have invalid ref sample, in witch case need to force take neighborhood to have a clamping box.
		#if DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION && CONFIG_UPSCALE
		{
			KernelConfig.bForceAllAccumulation = RefSamples.Array[0].SampleCount == 0;
			KernelConfig.SampleSet = SAMPLE_SET_3X3_PLUS;
		}
		#endif

		#if CONFIG_SAMPLE_SET == SAMPLE_SET_HEXAWEB
		{
			KernelConfig.RingCount = 1;

			// TODO(Denoiser): could be improved.
			//KernelConfig.bMinSamplePairInvFrequency = true;

			float2 E = float2(
				InterleavedGradientNoise(DispatchThreadId, 0),
				InterleavedGradientNoise(DispatchThreadId, 1));

			// Add a bit of jittering to hide low sample.
			KernelConfig.bSampleKernelCenter = false;
			KernelConfig.BufferUV += View.ViewSizeAndInvSize.zw * (E - 0.5) * (KernelConfig.KernelSpreadFactor);
		}
		#endif

		FSSDSignalAccumulatorArray UncompressedAccumulators = CreateSignalAccumulatorArray();

		// When not upscaling, manually force accumulate the sample of the kernel.
		if (!KernelConfig.bSampleKernelCenter && !KernelConfig.bDescOrder)
		{
			UNROLL_N(SIGNAL_ARRAY_SIZE)
			for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
			{
				const uint BatchedSignalId = ComputeSignalBatchIdFromSignalMultiplexId(KernelConfig, SignalMultiplexId);
				FSSDSignalDomainKnowledge DomainKnowledge = GetSignalDomainKnowledge(BatchedSignalId);

				uint2 RefPixelCoord = floor(KernelConfig.BufferUV * KernelConfig.BufferSizeAndInvSize.xy);
				FSSDSignalSample CenterSample = TransformSignalSampleForAccumulation(
					KernelConfig,
					SignalMultiplexId,
					RefSceneMetadata,
					RefSamples.Array[SignalMultiplexId],
					RefPixelCoord);
				
				FSSDSampleAccumulationInfos SampleInfos;
				SampleInfos.Sample = CenterSample;
				SampleInfos.Frequency = RefFrequencies.Array[SignalMultiplexId];
				SampleInfos.FinalWeight = 1.0;
				SampleInfos.InvFrequency = GetSignalWorldBluringRadius(SampleInfos.Frequency, RefSceneMetadata, DomainKnowledge);
				
				if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS)
				{
					SampleInfos.InvFrequency = SampleInfos.Frequency.WorldBluringRadius;
				}

				AccumulateSample(
					/* inout */ UncompressedAccumulators.Array[SignalMultiplexId],
					SampleInfos);
			}
		}

		#if CONFIG_SAMPLE_SET == SAMPLE_SET_STACKOWIAK_4_SETS
		{
			KernelConfig.SampleCount = clamp(uint(RequestedSampleCount) / kStackowiakSampleSetCount, 1, MaxSampleCount);

			#if CONFIG_UPSCALE
			{
				// TODO(Denoiser): could be optimised, but currently reusing same peace of code as reflection for maintainability.
				uint2 RayDispatchThreadId = (DispatchThreadId - UpscaleFactor / 2) / UpscaleFactor;
				uint2 ClosestRayPixelCoord = GetPixelCoord(RayDispatchThreadId);
			
				uint RaySubPixelId = View.StateFrameIndex & (UpscaleFactor * UpscaleFactor - 1);

				KernelConfig.BufferUV = ((ViewportMin + ClosestRayPixelCoord + (0.5 * KernelSpreadFactor + 0.5))) * KernelConfig.BufferSizeAndInvSize.zw;
			
				// Sample the center of the kernel by comparing it against the RefSceneMetadata, since it may no match.
				KernelConfig.bSampleKernelCenter = true;

				// Id of the pixel in the quad.
				KernelConfig.SampleTrackId = ((DispatchThreadId.x & 1) | ((DispatchThreadId.y & 1) << 1)) ^ 0x3;

				// To avoid precision problem when comparing potentially identicall 
				KernelConfig.bForceKernelCenterAccumulation = RaySubPixelId == ((DispatchThreadId.x & 1) | ((DispatchThreadId.y & 1) << 1));
			}
			#else
			{
				// Put the kernel center at the center of the quad. Half pixel shift is done in the sample offsets.
				KernelConfig.BufferUV = float2(DispatchThreadId | 1) * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;

				// Id of the pixel in the quad. This is to match hard coded first samples of the sample set.
				KernelConfig.SampleTrackId = ((DispatchThreadId.x & 1) | ((DispatchThreadId.y & 1) << 1));
			}
			#endif

			#if CONFIG_VGPR_OPTIMIZATION
				// Keek sample SampleTrackId & SceneBufferUV arround for computation of pixel output coordinate.
				// Should be VGPR free given it's curernt is being used in accumulation has well that is highest VGPR pressure of the shader.
				// TODO(Denoiser): could save 1 VGPR by using 2 SGPR instead of SampleTrackId.
				SampleTrackId = KernelConfig.SampleTrackId;
				KernelBufferUV = KernelConfig.BufferUV;
			#endif
		}
		#elif CONFIG_SAMPLE_SET == SAMPLE_SET_DIRECTIONAL_RECT || CONFIG_SAMPLE_SET == SAMPLE_SET_DIRECTIONAL_ELLIPSE
		{
			#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
			{
				const float TargetSamplePerPixel = 0.25;
				const float MinimalPixelRadius = 0.5 * rsqrt(2.0);

				// Project GGX lobe into screen space.
				float2 NormalizedScreenMajorAxis;
				float InifinityMajorViewportRadius;
				float InifinityMinorViewportRadius;
				ProjectSpecularLobeToScreenSpace(
					RefSceneMetadata,
					/* out */ NormalizedScreenMajorAxis,
					/* out */ InifinityMajorViewportRadius,
					/* out */ InifinityMinorViewportRadius);

				float ConfusionFactor = saturate(RefFrequencies.Array[0].ConfusionFactor);
				float AspectRatio = InifinityMinorViewportRadius / InifinityMajorViewportRadius;

				float PreviousMaxPixelDiameter = sqrt(rcp(TargetSamplePerPixel) * PreviousCumulativeMaxSampleCount / AspectRatio);
				float MaxPixelDiameter = sqrt(rcp(TargetSamplePerPixel) * MaxSampleCount * PreviousCumulativeMaxSampleCount / AspectRatio);

				KernelConfig.MajorAxis = NormalizedScreenMajorAxis * float2(1, -1);
				KernelConfig.MajorPixelRadius = InifinityMajorViewportRadius * ConfusionFactor * View.ViewSizeAndInvSize.x - PreviousMaxPixelDiameter;

				float MaxPixelRadius = 0.5 * MaxPixelDiameter;

				KernelConfig.MajorPixelRadius = clamp(KernelConfig.MajorPixelRadius, 0, MaxPixelRadius);
				KernelConfig.MinorPixelRadius = AspectRatio * KernelConfig.MajorPixelRadius;

				// *4 to multiply from radii area to diameters area.
				float ConvolutionArea = 4.0 * max(KernelConfig.MajorPixelRadius, MinimalPixelRadius) * max(KernelConfig.MinorPixelRadius, MinimalPixelRadius);

				KernelConfig.SampleCount = clamp(ConvolutionArea * TargetSamplePerPixel * rcp(PreviousCumulativeMaxSampleCount), 0, MaxSampleCount);
			
				#if 0
				{
					DebugOutput[DispatchThreadId] = float4(
						KernelConfig.SampleCount,
						KernelConfig.MajorPixelRadius,
						KernelConfig.MinorPixelRadius,
						ConfusionFactor);
				}
				#elif 0
				{
					// DebugOutput[DispatchThreadId] = float4(
					// 	KernelConfig.SampleCount,
					// 	InifinityMajorViewportRadius,
					// 	InifinityMinorViewportRadius,
					// 	AspectRatio);
				}
				#elif 0
				{
					DebugOutput[DispatchThreadId] = float4(
						//GetWorldNormal(RefSceneMetadata) * 0.5 + 0.5,
						abs(GetTranslatedWorldPosition(RefSceneMetadata) * 0.001),
						RefSceneMetadata.WorldDepth);
				}
				#endif

				// DebugOutput[DispatchThreadId] = float4(
				// 	KernelConfig.SampleCount,
				// 	KernelConfig.MajorPixelRadius,
				// 	KernelConfig.MinorPixelRadius,
				// 	OutOfFocus);
			}
			#else
				#error Directional rect sample set is not supported.
			#endif
		}
		#endif // CONFIG_SAMPLE_SET == SAMPLE_SET_DIRECTIONAL_*

		FSSDCompressedSignalAccumulatorArray CompressedAccumulators = CompressAccumulatorArray(UncompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
		

		// Performance: skip pixels/regions where the center sample is invalid (SampleCount = 0) for virtual shadow maps
		#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_VIRTUAL_SHADOW_MAP_MASK
		bool bRefHasSamples = false;
		for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
		{
			bRefHasSamples = bRefHasSamples || (RefSamples.Array[SignalMultiplexId].SampleCount > 0);
		}
		BRANCH
		if (bRefHasSamples)
		#endif
		{
			AccumulateKernel(
				KernelConfig,
				Signal_Textures_0,
				Signal_Textures_1,
				Signal_Textures_2,
				Signal_Textures_3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators);
		}

		// When doing history rejection pre convolution, could still have no information found with the 3x3 + kernel,
		// therefore dynamically complete to form an entire 3x3 convolution.
		#if DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION
		{
			BRANCH
			if (KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS &&
				KernelConfig.bForceAllAccumulation)
			{
				KernelConfig.SampleSet = SAMPLE_SET_3X3_CROSS;
				KernelConfig.bSampleKernelCenter = false;
			
				AccumulateKernel(
					KernelConfig,
					Signal_Textures_0,
					Signal_Textures_1,
					Signal_Textures_2,
					Signal_Textures_3,
					/* inout */ UncompressedAccumulators,
					/* inout */ CompressedAccumulators);
			}
		}
		#endif // DIM_STAGE == STAGE_REJECTION_PRE_CONVOLUTION
		
		// Manually sample the center of the kernel after any accumulation when accumulating in descending order.
		if (!KernelConfig.bSampleKernelCenter && KernelConfig.bDescOrder)
		{
			// Remove any jitter the kernel may have. Won't have ant VGPR cost when no jittering, because KernelConfig.BufferUV == SceneBufferUV.
			// TODO(Denoiser): This is costly for VGPR pressure if using KernelConfig.BufferUV was != SceneBufferUV.
			KernelConfig.BufferUV = SceneBufferUV;

			SampleAndAccumulateCenterSampleAsItsOwnCluster(
				KernelConfig,
				Signal_Textures_0,
				Signal_Textures_1,
				Signal_Textures_2,
				Signal_Textures_3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators);
		}
		
		#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION == ACCUMULATOR_COMPRESSION_DISABLED
			SignalAccumulators = UncompressedAccumulators;
		#else
			SignalAccumulators = UncompressAccumulatorArray(CompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
		#endif
	}

	// Color processing of the signal to reduce highlight flickering.
	#if CONFIG_ACCUMULATION_COLOR_SPACE != CONFIG_OUTPUT_COLOR_SPACE || CONFIG_REJECT_HIGHEST_COLOR
	{
		UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
		for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
		{
			UncompressSignalAccumulator(/* inout */ SignalAccumulators.Array[MultiplexId]);

			#if CONFIG_REJECT_HIGHEST_COLOR
			{
				#if !COMPILE_SIGNAL_COLOR
					#error Need to compile signal color.
				#endif
				if (Accumulator.Moment1[MultiplexId].SampleCount > 0)
				{
					const float MaxNeighborWeight = saturate(SignalAccumulators.Array[MultiplexId].Moment1.SampleCount * rcp(10) - 1);

					SignalAccumulators.Array[MultiplexId].Moment1.SceneColor.rgb =
						(SignalAccumulators.Array[MultiplexId].Moment1.SceneColor.rgb - MaxNeighbor.SceneColor.rgb * MaxNeighborWeight) *
						(SignalAccumulators.Array[MultiplexId].Moment1.SampleCount / (SignalAccumulators.Array[MultiplexId].Moment1.SampleCount - MaxNeighborWeight));
				}
			}
			#endif // CONFIG_REJECT_HIGHEST_COLOR

			#if CONFIG_ACCUMULATION_COLOR_SPACE != CONFIG_OUTPUT_COLOR_SPACE
			{
				#if COMPILE_MOMENT1_ACCUMULATOR
				SignalAccumulators.Array[MultiplexId].Moment1 = TransformSignal(
					SignalAccumulators.Array[MultiplexId].Moment1, 
					/* SrcBasis  = */ CONFIG_ACCUMULATION_COLOR_SPACE, 
					/* DestBasis = */ CONFIG_OUTPUT_COLOR_SPACE);
				#endif
				
				#if COMPILE_MOMENT2_ACCUMULATOR
				SignalAccumulators.Array[MultiplexId].Moment2 = TransformSignal(
					SignalAccumulators.Array[MultiplexId].Moment2, 
					/* SrcBasis  = */ CONFIG_ACCUMULATION_COLOR_SPACE, 
					/* DestBasis = */ CONFIG_OUTPUT_COLOR_SPACE);
				#endif
			}
			#endif // CONFIG_ACCUMULATION_COLOR_SPACE != CONFIG_OUTPUT_COLOR_SPACE

			// TODO(Denoiser): it might be better to just uncompress before this for loop and remain uncompressed,
			// so the color operation get done in practice during the output sample transcoding.
			CompressSignalAccumulator(/* inout */ SignalAccumulators.Array[MultiplexId]);
		}
	}
	#endif // CONFIG_ACCUMULATION_COLOR_SPACE != CONFIG_OUTPUT_COLOR_SPACE || CONFIG_REJECT_HIGHEST_COLOR

	// Transcode the spatial accumulation into multiplexed signal according to different modes.
	uint MultiplexCount = 1;
	FSSDSignalArray OutputSamples = CreateSignalArrayFromScalarValue(0.0);
	FSSDSignalFrequencyArray OutputFrequencies = CreateInvalidSignalFrequencyArray();
	{
		#if CONFIG_OUTPUT_MODE == OUTPUT_MODE_SUM
		{
			MultiplexCount = CONFIG_SIGNAL_BATCH_SIZE;
			
			UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
			for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
			{
				UncompressSignalAccumulator(/* inout */ SignalAccumulators.Array[MultiplexId]);

				OutputSamples.Array[MultiplexId] = SignalAccumulators.Array[MultiplexId].Moment1;
				
				// Output the minimal inverse frequency as new world bluring radius for subsequent passes.
				OutputFrequencies.Array[MultiplexId] = SignalAccumulators.Array[MultiplexId].MinFrequency;
			}
		}
		#elif CONFIG_OUTPUT_MODE == OUTPUT_MODE_2MOMMENT_SUM
		{
			#if SIGNAL_ARRAY_SIZE != 2 * MAX_SIGNAL_BATCH_SIZE
				#error Invalid signal array size.
			#endif

			MultiplexCount = 2 * CONFIG_SIGNAL_BATCH_SIZE;

			UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
			for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
			{
				UncompressSignalAccumulator(/* inout */ SignalAccumulators.Array[BatchedSignalId * CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN + 0]);

				OutputSamples.Array[BatchedSignalId * 2 + 0] = SignalAccumulators.Array[BatchedSignalId * CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN + 0].Moment1;
				OutputSamples.Array[BatchedSignalId * 2 + 1] = SignalAccumulators.Array[BatchedSignalId * CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN + 0].Moment2;
			}
		}
		#elif CONFIG_OUTPUT_MODE == OUTPUT_MODE_DRB
		{
			MultiplexCount = CONFIG_SIGNAL_BATCH_SIZE;
			
			UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
			for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
			{
				UncompressSignalAccumulator(/* inout */ SignalAccumulators.Array[MultiplexId]);

				OutputSamples.Array[MultiplexId] = SignalAccumulators.Array[MultiplexId].Previous;

				// Output the minimal inverse frequency as new world bluring radius for subsequent passes.
				OutputFrequencies.Array[MultiplexId] = SignalAccumulators.Array[MultiplexId].MinFrequency;
			
				// No need to keep the VGPR pressure at this point for WorldBluringRadius, because no passes use it after.
				if (DIM_STAGE == STAGE_POST_FILTERING && 0)
				{
					OutputFrequencies.Array[MultiplexId].WorldBluringRadius = 0;
				}
			}
		}
		#else
			#error Unknown output mode.
		#endif
	}
	
	// Clamp the number of sample recorded.
	#if DIM_STAGE == STAGE_POST_FILTERING
	{
		UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
		for (uint MultiplexId = 0; MultiplexId < CONFIG_SIGNAL_BATCH_SIZE; MultiplexId++)
		{
			float CurrentSampleCount = RefSamples.Array[MultiplexId].SampleCount;
			float NewSampleCount = min(CurrentSampleCount, TARGETED_SAMPLE_COUNT);

			OutputSamples.Array[MultiplexId] = MulSignal(OutputSamples.Array[MultiplexId], CurrentSampleCount > 0 ? NewSampleCount / CurrentSampleCount : 0);
		}
	}
	#endif // DIM_STAGE == STAGE_POST_FILTERING

	#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC && 0
		DebugOutput[DispatchThreadId] = float4(
			OutputSamples.Array[0].ColorSH.R.V.x,
			OutputSamples.Array[0].ColorSH.G.V.x,
			OutputSamples.Array[0].ColorSH.B.V.x,
			OutputSamples.Array[0].SampleCount);
	#endif

	// TODO(Denoiser):  LeaveRayCount = (LeaveRayCount - 1) * 9 / (9 - 2) post processing to reject when for history rejection
	uint2 OutputPixelPostion;
	#if CONFIG_VGPR_OPTIMIZATION && !CONFIG_UPSCALE // TODO(Denoiser)
	{
		// No need to keep DispatchThreadId, can recompute the output pixel position based on information stored in VGPRs for spatial kernel.
		#if CONFIG_SAMPLE_SET == SAMPLE_SET_STACKOWIAK_4_SETS
			#if CONFIG_UPSCALE
				SampleTrackId ^= 0x3;
			#endif
			OutputPixelPostion = (uint2(KernelBufferUV * BufferUVToOutputPixelPosition) & ~0x1) | (uint2(SampleTrackId, SampleTrackId >> 1) & 0x1);
		#else
			OutputPixelPostion = BufferUVToBufferPixelCoord(SceneBufferUV);
		#endif
	}
	#else
		OutputPixelPostion = ViewportMin + DispatchThreadId;
	#endif 
	
	#if DEBUG_OUTPUT && CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK && DIM_STAGE == STAGE_RECONSTRUCTION
		DebugOutput[DispatchThreadId] = float4(OutputSamples.Array[0].SampleCount, 0, 0, 0);
	#endif

	BRANCH
	if (all(OutputPixelPostion < ViewportMax))
	{
		// Output the multiplexed signal.
		#if DIM_STAGE == STAGE_FINAL_OUTPUT && (CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_VIRTUAL_SHADOW_MAP_MASK)
		{
			UNROLL
			for (uint MultiplexId = 0; MultiplexId < MultiplexCount; MultiplexId++)
			{
				float Shadow = GetSamplePenumbraSafe(OutputSamples.Array[MultiplexId]);

				const float ShadowFadeFraction = 1;
				float SSSTransmission = 0;

				#if (CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK && !CONFIG_DENOISE_TRANSMISSION)
				{
					
					if (MultiplexId == 0)
						SSSTransmission = OriginalInput_Textures_0[OutputPixelPostion].a;
					#if CONFIG_SIGNAL_BATCH_SIZE > 1
					else if (MultiplexId == 1)
						SSSTransmission = OriginalInput_Textures_1[OutputPixelPostion].a;
					#endif
					#if CONFIG_SIGNAL_BATCH_SIZE > 2
					else if (MultiplexId == 2)
						SSSTransmission = OriginalInput_Textures_2[OutputPixelPostion].a;
					#endif
					#if CONFIG_SIGNAL_BATCH_SIZE > 3
					else if (MultiplexId == 3)
						SSSTransmission = OriginalInput_Textures_3[OutputPixelPostion].a;
					#endif
						
				}
				#else
				{
					SSSTransmission = (OutputSamples.Array[MultiplexId].SampleCount > 0 ? OutputSamples.Array[MultiplexId].TransmissionDistance / OutputSamples.Array[MultiplexId].SampleCount : OutputSamples.Array[MultiplexId].TransmissionDistance);
				}
				#endif
				// 0 is shadowed, 1 is unshadowed
				// RETURN_COLOR not needed unless writing to SceneColor;
				float FadedShadow = lerp(1.0f, Shadow, ShadowFadeFraction);
				float FadedSSSShadow = lerp(1.0f, SSSTransmission, ShadowFadeFraction);

				// the channel assignment is documented in ShadowRendering.cpp (look for Light Attenuation channel assignment)
				float4 OutColor;
				if (GET_SCALAR_ARRAY_ELEMENT(LightType, MultiplexId) == LIGHT_TYPE_DIRECTIONAL)
				{
					OutColor = EncodeLightAttenuation(half4(FadedShadow, FadedSSSShadow, 1.0, FadedSSSShadow));
				}
				else
				{
					OutColor = EncodeLightAttenuation(half4(FadedShadow, FadedSSSShadow, FadedShadow, FadedSSSShadow));
				}

				if (MultiplexId == 0)
					SignalOutput_UAVs_0[OutputPixelPostion] = OutColor;
				if (MultiplexId == 1)
					SignalOutput_UAVs_1[OutputPixelPostion] = OutColor;
				if (MultiplexId == 2)
					SignalOutput_UAVs_2[OutputPixelPostion] = OutColor;
				if (MultiplexId == 3)
					SignalOutput_UAVs_3[OutputPixelPostion] = OutColor;
			}
		}
		#else
		{
			OutputMultiplexedSignal(
				SignalOutput_UAVs_0,
				SignalOutput_UAVs_1,
				SignalOutput_UAVs_2,
				SignalOutput_UAVs_3,
				CONFIG_SIGNAL_OUTPUT_LAYOUT,
				MultiplexCount,
				OutputPixelPostion,
				OutputSamples,
				OutputFrequencies);
		}
		#endif
	}
} // MainCS
