// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDSignalAccumulatorArray.ush"
#include "SSDSignalBufferEncoding.ush"
#include "../TextureSampling.ush"
#include "../MonteCarlo.ush"


//------------------------------------------------------- ENUMS

/** Enums to choose how to compute the world distance for bilateral rejection. */
	// Only depends on the reference sample's pixel size and depth.
	#define SIGNAL_WORLD_FREQUENCY_REF_METADATA_ONLY 0

	// Only depends on the sample's pixel size and depth.
	#define SIGNAL_WORLD_FREQUENCY_SAMPLE_METADATA_ONLY 1

	// Is the smallest according of pixel size and depth between reference and sample.
	#define SIGNAL_WORLD_FREQUENCY_MIN_METADATA 2

	// Depends only based of the sample's hit distance and metadata.
	#define SIGNAL_WORLD_FREQUENCY_HIT_DISTANCE 3

	// Uses FSSDSignalSample::WorldBluringRadius precomputed in the sample.
	#define SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS 4

	// Compute based on the harmonic being processed.
	#define SIGNAL_WORLD_FREQUENCY_HARMONIC 5


//------------------------------------------------------- CONFIG DISABLED DEFAULTS

#ifndef CONFIG_ACCUMULATOR_VGPR_COMPRESSION
	#define CONFIG_ACCUMULATOR_VGPR_COMPRESSION ACCUMULATOR_COMPRESSION_DISABLED
#endif

#define CONFIG_ENABLE_WAVE_BROADCAST (PLATFORM_SUPPORTS_WAVE_BROADCAST)


#ifndef COMPILE_BOX_KERNEL
	#define COMPILE_BOX_KERNEL 0
#endif

#ifndef COMPILE_STACKOWIAK_KERNEL
	#define COMPILE_STACKOWIAK_KERNEL 0
#endif

#ifndef COMPILE_DISK_KERNEL
	#define COMPILE_DISK_KERNEL 0
#endif

#ifndef COMPILE_DIRECTIONAL_KERNEL
	#define COMPILE_DIRECTIONAL_KERNEL 0
#endif

#ifndef COMPILE_RAW_EXPERIMENTAL_KERNEL
	#define COMPILE_RAW_EXPERIMENTAL_KERNEL 0
#endif

#ifndef FORCE_IDENTICAL_COLOR_SPACE
	#define FORCE_IDENTICAL_COLOR_SPACE 0
#endif

//------------------------------------------------------- STRUCTURES

/** Configures the spatial kernel. */
struct FSSDKernelConfig
{
	// --------------------------- compile time.

	// Compile time set of sample to use.
	uint SampleSet;
	
	// Compile time selection of sample to use.
	uint SampleSubSetId;

	// Compile time layout of the buffer to accumulate.
	uint BufferLayout;

	// Compile time number of multiplexed signal per signal domain.
	uint MultiplexedSignalsPerSignalDomain;
	
	// Selects how the world distance should be computed for bilateral rejection at compile time.
	uint BilateralDistanceComputation;

	// Number of ring for a disk kernel.
	uint RingCount;
	
	/** Selects how the computation of world vector between the reference and neighbor should be computed. */
	uint NeighborToRefComputation;
	
	// Layout of RefSceneMetadata
	uint RefSceneMetadataLayout;
	
	// Multiplier applied on the world bluring distance of the signal.
	float WorldBluringDistanceMultiplier;

	// Compile time configuration whether want do LOOP or UNROLL
	//  false by default to expose in user code when the shader byte code might potentially be big.
	bool bUnroll;
	
	// Compile time whether the center of the kernel sample is sampled.
	bool bSampleKernelCenter;

	// Compile time whether sampling previous frame or current frame metadata.
	bool bPreviousFrameMetadata;

	// Compile time whether reference metadata is current frame or previous frame.
	bool bPreviousFrameRefMetadata;

	// The sample should be accumulated starting from the further away.
	bool bDescOrder;

	// Whether a sample should be normalised to 1 before accmulation.
	bool bNormalizeSample;

	// Whether should min sample frequency of pair of samples
	// [ Jimenez 2014, "Next Generation Post Processing in Call of Duty: Advanced Warfare" ]
	bool bMinSamplePairInvFrequency;

	// Whether the bilateral distance should be maxed with reference bilateral distance.
	bool bMaxWithRefBilateralDistance;

	// Whether the spherical harmonic of a sample should be computed before accumulation.
	bool bComputeSampleColorSH;

	// Whether should clamp the UV individually per signal.
	bool bClampUVPerMultiplexedSignal;

	// The color space that has been encoded in the buffer.
	uint BufferColorSpace[SIGNAL_ARRAY_SIZE];
	
	// The color space of the accumulation.
	uint AccumulatorColorSpace[SIGNAL_ARRAY_SIZE];
	
	// The color space of the accumulation.
	uint BilateralSettings[SIGNAL_ARRAY_SIZE];

	
	// --------------------------- Per wave.

	// Buffer size and inv size.
	float4 BufferSizeAndInvSize;
	float4 BufferBilinearUVMinMax;

	// Multiplier on the sample's offset.
	float KernelSpreadFactor;

	// The periode of the harmonic being sampled.
	float HarmonicPeriode;

	// Buffer's min and max UV, per texture.
	float4 PerSignalUVMinMax[SIGNAL_ARRAY_SIZE];

	
	// --------------------------- Per lane.
	
	// Number of samples should be done when doing variable box sampling.
	uint BoxKernelRadius;

	// Runtime number of samples
	uint SampleCount;

	// Buffer coordinate of the center of the kernel.
	float2 BufferUV;

	// Metadata of the scene for the bilateral therm.
	FSSDCompressedSceneInfos CompressedRefSceneMetadata;

	// Buffer coordinate of the reference used for decompression.
	// Please try to make this same as BufferUV.
	float2 RefBufferUV;

	// Runtime to force the first sample of the kernel to be accumulated.
	bool bForceKernelCenterAccumulation;

	// Runtime to force accumulating all sample.
	bool bForceAllAccumulation;

	// Runtime whether this pixel is dynamic object.
	bool bIsDynamicPixel;

	// Runtime selection of a track of sample.
	uint SampleTrackId;

	// Reference meta data.
	float RefBilateralDistance[SIGNAL_ARRAY_SIZE];

	// Uniform random values required for stocastic kernel.
	float Randoms[1];

	// Seed for hamerley sequence used for stocastic kernel.
	uint2 HammersleySeed;

	// Normalized pixel space direction for directional kernels.
	float2 MajorAxis;

	// The pixel radius along the major and minor axes for directional kernels.
	float MajorPixelRadius;
	float MinorPixelRadius;

#if DEBUG_OUTPUT
	uint2 DebugPixelPosition;
	uint DebugEventCounter;
#endif
};

FSSDKernelConfig CreateKernelConfig()
{
	FSSDKernelConfig KernelConfig;
	KernelConfig.SampleSet = SAMPLE_SET_1X1;
	KernelConfig.SampleSubSetId = 0;
	KernelConfig.BufferLayout = SIGNAL_BUFFER_LAYOUT_UNINITIALIZED;
	KernelConfig.MultiplexedSignalsPerSignalDomain = SIGNAL_ARRAY_SIZE;
	KernelConfig.NeighborToRefComputation = NEIGHBOR_TO_REF_CACHE_WORLD_POSITION;
	KernelConfig.RefSceneMetadataLayout = METADATA_BUFFER_LAYOUT_DISABLED;
	KernelConfig.RingCount = 0;
	KernelConfig.WorldBluringDistanceMultiplier = 1.0;
	KernelConfig.bUnroll = false;
	KernelConfig.bSampleKernelCenter = false;
	KernelConfig.bPreviousFrameMetadata = false;
	KernelConfig.bPreviousFrameRefMetadata = false;
	KernelConfig.BilateralDistanceComputation = SIGNAL_WORLD_FREQUENCY_MIN_METADATA;
	KernelConfig.bDescOrder = false;
	KernelConfig.bNormalizeSample = false;
	KernelConfig.bMinSamplePairInvFrequency = false;
	KernelConfig.bMaxWithRefBilateralDistance = false;
	KernelConfig.bComputeSampleColorSH = false;
	KernelConfig.bClampUVPerMultiplexedSignal = false;
	
	{
		UNROLL_N(SIGNAL_ARRAY_SIZE)
		for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
		{
			KernelConfig.BufferColorSpace[MultiplexId] = STANDARD_BUFFER_COLOR_SPACE;
			KernelConfig.AccumulatorColorSpace[MultiplexId] = STANDARD_BUFFER_COLOR_SPACE;
			KernelConfig.BilateralSettings[MultiplexId] = 0x0000;
		}
	}
	
	// SGPRs.
	KernelConfig.BufferSizeAndInvSize = float4(0, 0, 0, 0);
	KernelConfig.BufferBilinearUVMinMax = float4(0, 0, 0, 0);
	KernelConfig.KernelSpreadFactor = 1;
	KernelConfig.HarmonicPeriode = 1.0;
	
	{
		UNROLL_N(SIGNAL_ARRAY_SIZE)
		for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
		{
			KernelConfig.PerSignalUVMinMax[MultiplexId] = 0.0;
		}
	}

	// VGPRs.
	KernelConfig.BoxKernelRadius = 1;
	KernelConfig.SampleCount = 1;
	KernelConfig.BufferUV = 0.0;
	KernelConfig.CompressedRefSceneMetadata = CreateCompressedSceneInfos();
	KernelConfig.RefBufferUV = 0.0;
	KernelConfig.bForceKernelCenterAccumulation = false;
	KernelConfig.bForceAllAccumulation = false;
	KernelConfig.bIsDynamicPixel = false;
	KernelConfig.SampleTrackId = 0;
	KernelConfig.MajorAxis = 0.0;
	KernelConfig.MajorPixelRadius = 0.0;
	KernelConfig.MinorPixelRadius = 0.0;
	KernelConfig.HammersleySeed = 0;

	{	
		UNROLL_N(SIGNAL_ARRAY_SIZE)
		for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
		{
			KernelConfig.RefBilateralDistance[MultiplexId] = 0.0;
		}
	}
	
	{	
		UNROLL_N(2)
		for (uint RandomSignalId = 0; RandomSignalId < 1; RandomSignalId++)
		{
			KernelConfig.Randoms[RandomSignalId] = 0.0;
		}
	}
	
	#if DEBUG_OUTPUT
	{
		KernelConfig.DebugPixelPosition = 0;
		KernelConfig.DebugEventCounter = 0;
	}
	#endif

	return KernelConfig;
}


void SetBilateralPreset(uint BilateralPresetId, inout FSSDKernelConfig KernelConfig)
{
	if (BilateralPresetId == BILATERAL_PRESET_MONOCHROMATIC_PENUMBRA)
	{
		UNROLL_N(SIGNAL_ARRAY_SIZE)
		for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
		{
			// Change the bilarteral settings to use normal orientation in 
			// order to not merge background / foreground sample, as otherwise this results into leaks
			// Shadow masks are normal invarient, so only reject based on position.
			KernelConfig.BilateralSettings[MultiplexId] = BILATERAL_POSITION_BASED(5) | BILATERAL_NORMAL;
		}
	}
	else if (BilateralPresetId == BILATERAL_PRESET_POLYCHROMATIC_PENUMBRA)
	{
		// Diffuse.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(5) | BILATERAL_NORMAL;
		
		// Specular.
		#if SIGNAL_ARRAY_SIZE > 1
			KernelConfig.BilateralSettings[1] = BILATERAL_POSITION_BASED(5) | BILATERAL_TOKOYASHI;
		#endif
	}
	else if (BilateralPresetId == BILATERAL_PRESET_REFLECTIONS)
	{
		// Specular.
		// Can only be done using tokoyashi because have more than one sample at a time.
		KernelConfig.BilateralSettings[0] = BILATERAL_TOKOYASHI;
		
		#if SIGNAL_ARRAY_SIZE > 1
			// Specular variance for sampling rejection pre convolution.
			KernelConfig.BilateralSettings[1] = KernelConfig.BilateralSettings[0];
		#endif
	}
	else if (BilateralPresetId == BILATERAL_PRESET_REFLECTIONS_1SPP)
	{
		// Specular.
		// Use specular ratio estomator, so no need to to reject based on the axis of the lobe.
		KernelConfig.BilateralSettings[0] = BILATERAL_TOKOYASHI_LOBE;
	}
	else if (BilateralPresetId == BILATERAL_PRESET_REFLECTIONS_TAA)
	{
		// Specular.
		// Can only be done using tokoyashi because have more than one sample at a time.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(1) | BILATERAL_TOKOYASHI;
		
		#if SIGNAL_ARRAY_SIZE > 1
			// Specular variance for sampling rejection pre convolution.
			KernelConfig.BilateralSettings[1] = KernelConfig.BilateralSettings[0];
		#endif
	}
	else if (BilateralPresetId == BILATERAL_PRESET_DIFFUSE)
	{
		// Diffuse depends on world position, but also the normal of the surface given it doesn't store any directionality.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(2) | BILATERAL_NORMAL;
		
		#if SIGNAL_ARRAY_SIZE > 1
			// Variance for sampling rejection pre convolution.
			KernelConfig.BilateralSettings[1] = KernelConfig.BilateralSettings[0];
		#endif
	}
	else if (BilateralPresetId == BILATERAL_PRESET_SPHERICAL_HARMONIC)
	{
		// Spherical harmonic encode directionality, so only reject based on world position.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(2);
	}
	else if (BilateralPresetId == BILATERAL_PRESET_PROBE_HIERARCHY)
	{
		// Diffuse & specular bilateral component.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(1) | BILATERAL_SHADING_MODEL;
	}
	else if (BilateralPresetId == BILATERAL_PRESET_AO)
	{
		// Diffuse depends on world position, but also the normal of the surface given it doesn't store any directionality.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(4) | BILATERAL_NORMAL;
		
		#if SIGNAL_ARRAY_SIZE > 1
			// Variance for sampling rejection pre convolution.
			KernelConfig.BilateralSettings[1] = KernelConfig.BilateralSettings[0];
		#endif
	}
	else if (BilateralPresetId == BILATERAL_PRESET_AO_HISTORY)
	{
		// Diffuse depends on world position, but also the normal of the surface given it doesn't store any directionality.
		KernelConfig.BilateralSettings[0] = BILATERAL_POSITION_BASED(2) | BILATERAL_NORMAL;
		//KernelConfig.BilateralSettings[0] = BILATERAL_NORMAL;
		
		#if SIGNAL_ARRAY_SIZE > 1
			// Variance for sampling rejection pre convolution.
			KernelConfig.BilateralSettings[1] = KernelConfig.BilateralSettings[0];
		#endif
	}
}


//------------------------------------------------------- CONSTANT

static const float kWaveletFilterWeights5x5[] = { 3.0 / 8.0, 1.0 / 4.0, 1.0 / 16.0 };


//------------------------------------------------------- REDERIVE INFORMATION FOR LOWER VGPR OCCUPENCY

/** Deduce the buffer UV of the output pixel this kernel has been configured for. */
ISOLATE
float2 ComputeRefBufferUV(FSSDKernelConfig KernelConfig)
{
	if (KernelConfig.bPreviousFrameMetadata)
	{
		// Impossible to compute from BufferUV because it's in the previous frame basis.
		return KernelConfig.RefBufferUV;
	}
	else if (KernelConfig.SampleSet == SAMPLE_SET_HEXAWEB)
	{
		// Impossible to compute from BufferUV because of random offset certainely needed using this..
		return KernelConfig.RefBufferUV;
	}
	else if (KernelConfig.SampleSet == SAMPLE_SET_STACKOWIAK_4_SETS)
	{
		uint SampleTrackId = KernelConfig.SampleTrackId;

		// Matches first line of kStackowiakSampleSet0
		// TODO(Denoiser): could be optimised further by just setting sign bit on 0.5.
		float2 SampleOffset = float2(
			SampleTrackId & 0x1 ? 0.5 : -0.5,
			SampleTrackId & 0x2 ? 0.5 : -0.5);

		return KernelConfig.BufferUV + SampleOffset * KernelConfig.BufferSizeAndInvSize.zw;
	}

	return KernelConfig.BufferUV;
}

/** Uncompress the reference scene metadata to keep a low VGPR pressure. */
ISOLATE
FSSDSampleSceneInfos UncompressRefSceneMetadata(FSSDKernelConfig KernelConfig)
{
	// Find out the buffer UV of the reference pixel.
	float2 RefBufferUV = ComputeRefBufferUV(KernelConfig);

	float2 ScreenPos;
	if (KernelConfig.bPreviousFrameMetadata) // TODO(Denoiser): should be bPreviousFrameRefMetadata instead?
	{
		ScreenPos = RefBufferUV * PrevSceneBufferUVToScreenPosition.xy + PrevSceneBufferUVToScreenPosition.zw;
	}
	else
	{
		ScreenPos = DenoiserBufferUVToScreenPosition(RefBufferUV);
	}

	// Uncompress the reference scene metadata to keep a low VGPR pressure.
	return UncompressSampleSceneInfo(
		KernelConfig.RefSceneMetadataLayout, KernelConfig.bPreviousFrameRefMetadata,
		ScreenPos,
		KernelConfig.CompressedRefSceneMetadata);
}

/** Uncompress the scene metadata of a sample. */
FSSDSampleSceneInfos UncompressSampleSceneMetadata(
	FSSDKernelConfig KernelConfig,
	float2 SampleBufferUV,
	FSSDCompressedSceneInfos CompressedSampleSceneMetadata)
{
	float2 ScreenPos;
	if (KernelConfig.bPreviousFrameMetadata)
	{
		ScreenPos = SampleBufferUV * PrevSceneBufferUVToScreenPosition.xy + PrevSceneBufferUVToScreenPosition.zw;
	}
	else
	{
		ScreenPos = DenoiserBufferUVToScreenPosition(SampleBufferUV);
	}

	return UncompressSampleSceneInfo(
		CONFIG_METADATA_BUFFER_LAYOUT, KernelConfig.bPreviousFrameMetadata,
		ScreenPos,
		CompressedSampleSceneMetadata);
}

float3 ComputeVectorFromNeighborToRef(
	FSSDKernelConfig KernelConfig,
	FSSDSampleSceneInfos RefSceneMetadata,
	FSSDSampleSceneInfos NeighborSceneMetadata)
{
	float RefWorldDepth = GetWorldDepth(RefSceneMetadata);
	float NeighborWorldDepth = GetWorldDepth(NeighborSceneMetadata);

	if (KernelConfig.NeighborToRefComputation == NEIGHBOR_TO_REF_LOWEST_VGPR_PRESSURE)
	{
		// Recompute the the screen position of the reference, from the most minimal VGPR footprint.
		float2 RefScreenPos = RefSceneMetadata.ScreenPosition;
		float3 RefClipPosition = float3(GetScreenPositionForProjectionType(RefScreenPos, RefWorldDepth), RefWorldDepth);

		float2 NeighborScreenPos = NeighborSceneMetadata.ScreenPosition;
		float3 NeighborClipPosition = float3(GetScreenPositionForProjectionType(NeighborScreenPos, NeighborWorldDepth), NeighborWorldDepth);
		
		#if CONFIG_USE_VIEW_SPACE
			float3 NeighborToRefVector = mul(float4(RefClipPosition - NeighborClipPosition, 0), GetScreenToViewDistanceMatrix()).xyz;
		#else
			float3 NeighborToRefVector = mul(float4(RefClipPosition - NeighborClipPosition, 0), View.ScreenToTranslatedWorld).xyz;
		#endif

		return NeighborToRefVector;
	}
	else // if (KernelConfig.NeighborToRefComputation == NEIGHBOR_TO_REF_CACHE_WORLD_POSITION)
	{
		float3 NeighborToRefWorldVector = GetTranslatedWorldPosition(RefSceneMetadata) - GetTranslatedWorldPosition(NeighborSceneMetadata);
		
		// TODO(Denoiser): GetViewPosition(RefSceneMetadata)
		#if CONFIG_USE_VIEW_SPACE
			return mul(float4(NeighborToRefWorldVector, 0), View.TranslatedWorldToView).xyz;
		#endif

		return NeighborToRefWorldVector;
	}
}


//------------------------------------------------------- SHARED SAMPLING

FSSDSignalSample TransformSignalSampleForAccumulation(
	FSSDKernelConfig KernelConfig,
	uint MultiplexId,
	FSSDSampleSceneInfos SampleSceneMetadata,
	FSSDSignalSample Sample,
	uint2 SamplePixelCoord)
{
	// Transform the color space.
	#if (!FORCE_IDENTICAL_COLOR_SPACE)
	// TODO(Denoiser): could pass down information that this sample may be normalized.
	Sample = TransformSignal(
		Sample, 
		/* SrcBasis  = */ KernelConfig.BufferColorSpace[MultiplexId], 
		/* DestBasis = */ KernelConfig.AccumulatorColorSpace[MultiplexId]);
	#endif

	// Compute the spherical harmonic of the sample.
	#if COMPILE_SIGNAL_COLOR_SH && COMPILE_SIGNAL_COLOR
	if (KernelConfig.bComputeSampleColorSH)
	{
		Sample.ColorSH = ComputeSampleColorSH(SampleSceneMetadata, Sample, SamplePixelCoord);
	}
	#endif

	return Sample;
}

/** Compute at compile time the index of the signal in the batch, from the index of the multiplexed signal. */
uint ComputeSignalBatchIdFromSignalMultiplexId(FSSDKernelConfig KernelConfig, const uint SignalMultiplexId)
{
	return SignalMultiplexId / KernelConfig.MultiplexedSignalsPerSignalDomain;
}

/** Returns whether this sample is outside the viewport. */
bool IsOutsideViewport(FSSDKernelConfig KernelConfig, float2 SampleBufferUV)
{
	return any(or(SampleBufferUV < KernelConfig.BufferBilinearUVMinMax.xy, SampleBufferUV > KernelConfig.BufferBilinearUVMinMax.zw));
}

/** Sample multiplexed samples and their metadata for kernel use. */
void SampleMultiplexedSignals(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	float2 SampleBufferUV,
	out FSSDCompressedSceneInfos OutCompressedSampleSceneMetadata,
	out FSSDCompressedMultiplexedSample OutCompressedMultiplexedSamples)
{
	uint2 PixelCoord = BufferUVToBufferPixelCoord(SampleBufferUV);

	OutCompressedSampleSceneMetadata = SampleCompressedSceneMetadata(
		KernelConfig.bPreviousFrameMetadata, SampleBufferUV, PixelCoord);

	// Fetches the signals sample
	OutCompressedMultiplexedSamples = SampleCompressedMultiplexedSignals(
		SignalBuffer0,
		SignalBuffer1,
		SignalBuffer2,
		SignalBuffer3,
		GlobalPointClampedSampler,
		SampleBufferUV,
		PixelCoord);
} // SampleMultiplexedSignals()

/** Uncompressed multiplexed signal for accumulation. */
void UncompressMultiplexedSignals(
	FSSDKernelConfig KernelConfig,
	float2 SampleBufferUV,
	FSSDCompressedMultiplexedSample CompressedMultiplexedSamples,
	out FSSDSignalArray MultiplexedSamples,
	out FSSDSignalFrequencyArray MultiplexedFrequencies)
{
	// TODO(Denoiser): offer multiplier to apply to each signal during Decode, to save mul VALU.
	DecodeMultiplexedSignals(
		KernelConfig.BufferLayout,
		/* MultiplexedSampleId = */ 0,
		KernelConfig.bNormalizeSample,
		CompressedMultiplexedSamples,
		/* out */ MultiplexedSamples,
		/* out */ MultiplexedFrequencies);

	if (KernelConfig.bClampUVPerMultiplexedSignal)
	{
		UNROLL_N(SIGNAL_ARRAY_SIZE)
		for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
		{
			bool bInvalidSample = any(SampleBufferUV != clamp(
				SampleBufferUV, KernelConfig.PerSignalUVMinMax[SignalMultiplexId].xy, KernelConfig.PerSignalUVMinMax[SignalMultiplexId].zw));

			if (bInvalidSample)
			{
				MultiplexedSamples.Array[SignalMultiplexId] = CreateSignalSampleFromScalarValue(0.0);
			}
		} // for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
	}
}

/** Accumulate multiplexed samples and their metadata to an accumulator. */
void AccumulateSampledMultiplexedSignals(
	FSSDKernelConfig KernelConfig,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	FSSDSampleSceneInfos RefSceneMetadata,
	float2 SampleBufferUV,
	FSSDSampleSceneInfos SampleSceneMetadata,
	FSSDSignalArray MultiplexedSamples,
	FSSDSignalFrequencyArray MultiplexedFrequencies,
	float KernelSampleWeight,
	const bool bForceSample,
	bool bIsOutsideFrustum)
{
	// Compute the bluring radius of the output pixel itself.
	float RefPixelWorldBluringRadius = ComputeWorldBluringRadiusCausedByPixelSize(RefSceneMetadata);

	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		FSSDSignalAccumulatorArray Accumulators = UncompressAccumulatorArray(CompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif
		
	// Compute the vector from neighbor to reference in the most optimal way.
	float3 NeighborToRefVector = ComputeVectorFromNeighborToRef(
		KernelConfig,
		RefSceneMetadata,
		SampleSceneMetadata);

	#if DEBUG_OUTPUT && 0
	if (KernelConfig.DebugEventCounter)
	{
		float4 A = float4(
			RefSceneMetadata.WorldDepth,
			SampleSceneMetadata.WorldDepth,
			length(NeighborToRefVector) / RefPixelWorldBluringRadius,
			KernelConfig.bPreviousFrameMetadata);

		float4 B = float4(
			DenoiserBufferUVToScreenPosition(SampleBufferUV) * 0.5 + 0.5,
			0,
			0);
		
		float4 C = float4(
			100 * abs(RefSceneMetadata.WorldDepth - SampleSceneMetadata.WorldDepth),
			0,
			0,
			0);
		
		float4 D = float4(
			length(RefSceneMetadata.TranslatedWorldPosition - SampleSceneMetadata.TranslatedWorldPosition),
			abs(RefSceneMetadata.WorldDepth - SampleSceneMetadata.WorldDepth),
			length(RefSceneMetadata.ScreenPosition - SampleSceneMetadata.ScreenPosition),
			0);

		DebugOutput[KernelConfig.DebugPixelPosition] = A;

		KernelConfig.DebugEventCounter = 0;
	}
	#endif

	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
	{
		// Compute at compile time the id of the signal being processed.
		const uint BatchedSignalId = ComputeSignalBatchIdFromSignalMultiplexId(KernelConfig, SignalMultiplexId);
		
		// Domain knowledge of the signal.
		FSSDSignalDomainKnowledge DomainKnowledge = GetSignalDomainKnowledge(BatchedSignalId);

		// TODO(Denoiser): direction of the ray should be cached by injest or output by RGS, otherwise ends up with VGPR pressure because of SampleBufferUV.
		uint2 NeighborPixelCoord = floor(SampleBufferUV * KernelConfig.BufferSizeAndInvSize.xy);

		// Fetch and pre process the sample for accumulation.
		FSSDSignalSample Sample = MultiplexedSamples.Array[SignalMultiplexId];
		Sample = TransformSignalSampleForAccumulation(KernelConfig, SignalMultiplexId, SampleSceneMetadata, Sample, NeighborPixelCoord);

		// Fetch sample's frequency for accumulation.
		FSSDSignalFrequency SampleFrequency = MultiplexedFrequencies.Array[SignalMultiplexId];

		// Compute the bluring radius of pixel itself.
		float SamplePixelWorldBluringRadius = ComputeWorldBluringRadiusCausedByPixelSize(SampleSceneMetadata);
		
		// Compute the bluring radius of the signal from ray hit distance and signal domain knowledge.
		float SignalConvolutionBluringRadius = GetSignalWorldBluringRadius(SampleFrequency, SampleSceneMetadata, DomainKnowledge);
		
		// But the signal's bluring radius might already be pre computed.
		if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS)
		{
			SignalConvolutionBluringRadius = SampleFrequency.WorldBluringRadius;
		}

		// Compute the final world distance to use for bilateral rejection.
		float FinalWorldBluringDistance = -1;
		if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_REF_METADATA_ONLY)
		{
			FinalWorldBluringDistance = AmendWorldBluringRadiusCausedByPixelSize(
				RefPixelWorldBluringRadius);
		}
		else if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_SAMPLE_METADATA_ONLY)
		{
			FinalWorldBluringDistance = AmendWorldBluringRadiusCausedByPixelSize(
				SamplePixelWorldBluringRadius);
		}
		else if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_MIN_METADATA)
		{
			FinalWorldBluringDistance = AmendWorldBluringRadiusCausedByPixelSize(
				min(SamplePixelWorldBluringRadius, RefPixelWorldBluringRadius));
		}
		else if (
			KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_HIT_DISTANCE ||
			KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS)
		{
			FinalWorldBluringDistance = SignalConvolutionBluringRadius;
		}
		else if (KernelConfig.BilateralDistanceComputation == SIGNAL_WORLD_FREQUENCY_HARMONIC)
		{
			FinalWorldBluringDistance = AmendWorldBluringRadiusCausedByPixelSize(
				RefPixelWorldBluringRadius) * KernelConfig.HarmonicPeriode;
		}

		FinalWorldBluringDistance *= KernelConfig.WorldBluringDistanceMultiplier;
		
		if (KernelConfig.bMaxWithRefBilateralDistance)
		{
			FinalWorldBluringDistance = min(FinalWorldBluringDistance, KernelConfig.RefBilateralDistance[SignalMultiplexId]);
		}
	
		// Compute the weight to be applied to do bilateral rejection.
		float BilateralWeight = ComputeBilateralWeight(
			KernelConfig.BilateralSettings[SignalMultiplexId],
			FinalWorldBluringDistance,
			DomainKnowledge,
			RefSceneMetadata,
			SampleSceneMetadata,
			NeighborToRefVector);
		
		FSSDSampleAccumulationInfos SampleInfos;
		SampleInfos.Sample = Sample;
		SampleInfos.Frequency = SampleFrequency;
		SampleInfos.FinalWeight = KernelSampleWeight * BilateralWeight;
		SampleInfos.InvFrequency = SignalConvolutionBluringRadius;

		if (bForceSample || KernelConfig.bForceAllAccumulation)
		{
			SampleInfos.FinalWeight = 1;
		}

		// TODO(Denoiser): bIsOutsideFrustum could afect number of samples for DRB.
		FLATTEN
		if (SampleInfos.Sample.SampleCount != 0 && !bIsOutsideFrustum)
		{
			#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION == ACCUMULATOR_COMPRESSION_DISABLED
			{
				AccumulateSample(
					/* inout */ UncompressedAccumulators.Array[SignalMultiplexId],
					SampleInfos);
			}
			#else
			{
				AccumulateSample(
					/* inout */ Accumulators.Array[SignalMultiplexId],
					SampleInfos);
			}
			#endif
		}
	} // for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
	
	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		CompressedAccumulators = CompressAccumulatorArray(Accumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif
} // AccumulateSampledMultiplexedSignals().

/** Sample and accumulate to accumulatore array.
 * 
 * Caution: you probably want to explicitly do this manually to help the shader compiler to do lattency hiding.
 */
void SampleAndAccumulateMultiplexedSignals(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	float2 SampleBufferUV,
	float KernelSampleWeight,
	const bool bForceSample)
{
	// Stores in SGPR whether this sample is outside the viewport, to avoid VGPR pressure to keep SampleBufferUV after texture fetches.
	bool bIsOutsideFrustum = IsOutsideViewport(KernelConfig, SampleBufferUV);

	FSSDCompressedSceneInfos CompressedSampleSceneMetadata;
	FSSDCompressedMultiplexedSample CompressedMultiplexedSamples;

	// Force all the signal texture fetch and metadata to overlap to minimize serial texture fetches.
	ISOLATE
	{
		SampleMultiplexedSignals(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			SampleBufferUV,
			/* out */ CompressedSampleSceneMetadata,
			/* out */ CompressedMultiplexedSamples);
	}
	
	// Accumulate the samples, giving full freedom for shader compiler scheduler to put instructions in most optimal way.
	{
		FSSDSignalArray MultiplexedSamples;
		FSSDSignalFrequencyArray MultiplexedFrequencies;
		UncompressMultiplexedSignals(
			KernelConfig, SampleBufferUV, CompressedMultiplexedSamples,
			/* out */ MultiplexedSamples,
			/* out */ MultiplexedFrequencies);
	
		FSSDSampleSceneInfos RefSceneMetadata = UncompressRefSceneMetadata(KernelConfig);
	
		FSSDSampleSceneInfos SampleSceneMetadata = UncompressSampleSceneMetadata(
			KernelConfig, SampleBufferUV, CompressedSampleSceneMetadata);

		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			MultiplexedSamples,
			MultiplexedFrequencies,
			KernelSampleWeight,
			bForceSample,
			bIsOutsideFrustum);
	}
} // SampleAndAccumulateMultiplexedSignals()

void SampleAndAccumulateMultiplexedSignalsPair(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	float2 SampleBufferUV[2],
	float KernelSampleWeight)
{
	FSSDCompressedSceneInfos CompressedSampleSceneMetadata[2];
	FSSDCompressedMultiplexedSample CompressedMultiplexedSamples[2];
	bool bIsOutsideFrustum[2];
	
	// Force all the signal texture fetch and metadata to overlap to minimize serial texture fetches.
	ISOLATE
	{
		UNROLL_N(2)
		for (uint PairFetchId = 0; PairFetchId < 2; PairFetchId++)
		{
			// Stores in SGPR whether this sample is outside the viewport, to avoid VGPR pressure to
			// avoid keeping SampleBufferUV after texture fetches.
			bIsOutsideFrustum[PairFetchId] = IsOutsideViewport(KernelConfig, SampleBufferUV[PairFetchId]);

			SampleMultiplexedSignals(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				SampleBufferUV[PairFetchId],
				/* out */ CompressedSampleSceneMetadata[PairFetchId],
				/* out */ CompressedMultiplexedSamples[PairFetchId]);
		}
	}
	
	// Accumulate the samples, giving full freedom for shader compiler scheduler to put instructions in most optimal way.
	{
		// Uncompress the multiplexed signal.
		FSSDSignalArray MultiplexedSamples[2];
		FSSDSignalFrequencyArray MultiplexedFrequencies[2];
		UNROLL_N(2)
		for (uint PairUncompressId = 0; PairUncompressId < 2; PairUncompressId++)
		{
			UncompressMultiplexedSignals(
				KernelConfig,
				SampleBufferUV[PairUncompressId],
				CompressedMultiplexedSamples[PairUncompressId],
				/* out */ MultiplexedSamples[PairUncompressId],
				/* out */ MultiplexedFrequencies[PairUncompressId]);
		}

		// Take the min inverse frequency per signal if desired.
		if (KernelConfig.bMinSamplePairInvFrequency)
		{
			UNROLL_N(SIGNAL_ARRAY_SIZE)
			for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
			{
				float MinInvFrequency = min(
					MultiplexedFrequencies[0].Array[SignalMultiplexId].WorldBluringRadius,
					MultiplexedFrequencies[1].Array[SignalMultiplexId].WorldBluringRadius);

				FLATTEN
				if (MinInvFrequency > 0)
				{
					MultiplexedFrequencies[0].Array[SignalMultiplexId].WorldBluringRadius = MinInvFrequency;
					MultiplexedFrequencies[1].Array[SignalMultiplexId].WorldBluringRadius = MinInvFrequency;
				}
			}
		}
	
		FSSDSampleSceneInfos RefSceneMetadata = UncompressRefSceneMetadata(KernelConfig);
	
		UNROLL_N(2)
		for (uint PairAccumulateId = 0; PairAccumulateId < 2; PairAccumulateId++)
		{
			FSSDSampleSceneInfos SampleSceneMetadata = UncompressSampleSceneMetadata(
				KernelConfig, SampleBufferUV[PairAccumulateId], CompressedSampleSceneMetadata[PairAccumulateId]);

			AccumulateSampledMultiplexedSignals(
				KernelConfig,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				RefSceneMetadata,
				SampleBufferUV[PairAccumulateId],
				SampleSceneMetadata,
				MultiplexedSamples[PairAccumulateId],
				MultiplexedFrequencies[PairAccumulateId],
				KernelSampleWeight,
				/* bForceSample = */ false,
				bIsOutsideFrustum[PairAccumulateId]);
		}
	}
} // SampleAndAccumulateMultiplexedSignalsPair()

void StartAccumulatingCluster(
	FSSDKernelConfig KernelConfig,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	FSSDSampleClusterInfo ClusterInfo)
{
	FSSDSampleSceneInfos RefSceneMetadata = UncompressRefSceneMetadata(KernelConfig);

	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		FSSDSignalAccumulatorArray Accumulators = UncompressAccumulatorArray(CompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif

	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
	{
		#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION == ACCUMULATOR_COMPRESSION_DISABLED
		{
			StartAccumulatingCluster(
				RefSceneMetadata,
				/* inout */ UncompressedAccumulators.Array[SignalMultiplexId],
				ClusterInfo);
		}
		#else
		{
			StartAccumulatingCluster(
				RefSceneMetadata,
				/* inout */ Accumulators.Array[SignalMultiplexId],
				ClusterInfo);
		}
		#endif
	}
	
	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		CompressedAccumulators = CompressAccumulatorArray(Accumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif
}

void DijestAccumulatedClusterSamples(
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	uint RingId, uint SampleCount)
{
	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		FSSDSignalAccumulatorArray Accumulators = UncompressAccumulatorArray(CompressedAccumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif
		
	UNROLL_N(SIGNAL_ARRAY_SIZE)
	for (uint SignalMultiplexId = 0; SignalMultiplexId < SIGNAL_ARRAY_SIZE; SignalMultiplexId++)
	{
		#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION == ACCUMULATOR_COMPRESSION_DISABLED
		{
			DijestAccumulatedClusterSamples(
				/* inout */ UncompressedAccumulators.Array[SignalMultiplexId],
				RingId, SampleCount);
		}
		#else
		{
			DijestAccumulatedClusterSamples(
				/* inout */ Accumulators.Array[SignalMultiplexId],
				RingId, SampleCount);
		}
		#endif
	}
	
	#if CONFIG_ACCUMULATOR_VGPR_COMPRESSION != ACCUMULATOR_COMPRESSION_DISABLED
		CompressedAccumulators = CompressAccumulatorArray(Accumulators, CONFIG_ACCUMULATOR_VGPR_COMPRESSION);
	#endif
}

void SampleAndAccumulateCenterSampleAsItsOwnCluster(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	const uint RingId = 0;

	FSSDSampleClusterInfo ClusterInfo;
	ClusterInfo.OutterBoundaryRadius = (RingId + 1) * KernelConfig.KernelSpreadFactor;

	StartAccumulatingCluster(
		KernelConfig, 
		/* inout */ UncompressedAccumulators,
		/* inout */ CompressedAccumulators,
		ClusterInfo);

	SampleAndAccumulateMultiplexedSignals(
		KernelConfig,
		SignalBuffer0,
		SignalBuffer1,
		SignalBuffer2,
		SignalBuffer3,
		/* inout */ UncompressedAccumulators,
		/* inout */ CompressedAccumulators,
		KernelConfig.BufferUV,
		/* KernelSampleWeight = */ 1.0,
		/* bForceSample = */ KernelConfig.bForceKernelCenterAccumulation);

	DijestAccumulatedClusterSamples(
		/* inout */ UncompressedAccumulators,
		/* inout */ CompressedAccumulators,
		RingId, /* SampleCount = */ 1);
}


//------------------------------------------------------- EASY CONVOLUTIONS

#if COMPILE_BOX_KERNEL

void AccumulateBilinear2x2Kernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	const float MipLevelPow2 = 1;

	FBilinearSampleInfos BilinearInfos = GetBilinearSampleLevelInfosEx(
		KernelConfig.BufferUV,
		KernelConfig.BufferSizeAndInvSize.xy,
		KernelConfig.BufferSizeAndInvSize.zw,
		MipLevelPow2, rcp(MipLevelPow2));
	
	bool bUseStocasticBilinear = false;
	if (KernelConfig.SampleSet == SAMPLE_SET_2X2_STOCASTIC)
	{
		bUseStocasticBilinear = true;
	}
	else if (KernelConfig.SampleSet == SAMPLE_SET_2X2_ADAPTIVE)
	{
		bUseStocasticBilinear = !KernelConfig.bIsDynamicPixel;
	}

	float2 SampleBufferUVArray[4];
	float BilinearWeightArray[4];

	FLATTEN
	if (bUseStocasticBilinear)
	{
		float2 SampleOffset = 0;
		float WeigthAccumulation = 0.0;

		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			FLATTEN
			if (KernelConfig.Randoms[0] > WeigthAccumulation)
				SampleOffset = BilinearSamplingOffsets2x2[i];

			WeigthAccumulation += GetSampleWeight(BilinearInfos, i);

			BilinearWeightArray[i] = 0.0;
			SampleBufferUVArray[i] = 0.0;
		}

		// TODO(Denoiser): could be more ALU efficient for this.
		// TODO(Denoiser): -0.5 full res pixel to ensure always select the mip, regardless of mantissa precision?
		SampleBufferUVArray[0] = (BilinearInfos.TopLeftPixelCoord + (SampleOffset + 0.5)) * MipLevelPow2 * KernelConfig.BufferSizeAndInvSize.zw;
		BilinearWeightArray[0] = 1.0;
	}
	else
	{
		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			float2 SampleOffset = BilinearSamplingOffsets2x2[i];
		
			// TODO(Denoiser): could be more ALU efficient for this.
			// TODO(Denoiser): -0.5 full res pixel to ensure always select the mip, regardless of mantissa precision?
			SampleBufferUVArray[i] = (BilinearInfos.TopLeftPixelCoord + (SampleOffset + 0.5)) * MipLevelPow2 * KernelConfig.BufferSizeAndInvSize.zw;
		
			BilinearWeightArray[i] = GetSampleWeight(BilinearInfos, i);
		}
	}

	{
		SampleAndAccumulateMultiplexedSignals(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUVArray[0],
			BilinearWeightArray[0],
			/* bForceSample = */ false);
	}

	BRANCH
	if (!bUseStocasticBilinear)
	{
		UNROLL_N(3)
		for (uint i = 1; i < 4; i++)
		{
			SampleAndAccumulateMultiplexedSignals(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				SampleBufferUVArray[i],
				BilinearWeightArray[i],
				/* bForceSample = */ false);
		}
	}
} // AccumulateBilinear2x2Kernel()

void AccumulateSquareKernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	int KernelRadius = 1;
	if (KernelConfig.SampleSet == SAMPLE_SET_5X5_WAVELET)
	{
		KernelRadius = 2;
	}
	else if (KernelConfig.SampleSet == SAMPLE_SET_NXN)
	{
		KernelRadius = KernelConfig.BoxKernelRadius;
	}
	
	if (KernelConfig.bUnroll)
	{
		UNROLL for (int x = -KernelRadius; x <= KernelRadius; x++)
		{
			UNROLL for (int y = -KernelRadius; y <= KernelRadius; y++)
			{
				const bool bIsKernelCenterSample = x == 0 && y == 0;

				if (bIsKernelCenterSample && !KernelConfig.bSampleKernelCenter) continue;

				float2 SampleOffset = float2(x, y);
				if (KernelConfig.SampleSet == SAMPLE_SET_3X3_SOBEK2018)
				{
					SampleOffset = mul(float2x2(float2(2, -1), float2(1, 2)), SampleOffset);
				}
			
				float2 SampleBufferUV = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
		
				float KernelWeight = 1;
				if (KernelConfig.SampleSet == SAMPLE_SET_5X5_WAVELET)
				{
					KernelWeight = 
						kWaveletFilterWeights5x5[abs(x)] *
						kWaveletFilterWeights5x5[abs(y)] *
						rcp(kWaveletFilterWeights5x5[0] * kWaveletFilterWeights5x5[0]);
				}

				SampleAndAccumulateMultiplexedSignals(
					KernelConfig,
					SignalBuffer0,
					SignalBuffer1,
					SignalBuffer2,
					SignalBuffer3,
					/* inout */ UncompressedAccumulators,
					/* inout */ CompressedAccumulators,
					SampleBufferUV,
					KernelWeight,
					/* bForceSample = */ bIsKernelCenterSample && KernelConfig.bForceKernelCenterAccumulation);
			}
		}
	}
	else
	{
		// TODO(Denoiser): latency hiding of this is terrible.
		LOOP for (int x = -KernelRadius; x <= KernelRadius; x++)
		{
			LOOP for (int y = -KernelRadius; y <= KernelRadius; y++)
			{
				const bool bIsKernelCenterSample = x == 0 && y == 0;

				if (bIsKernelCenterSample && !KernelConfig.bSampleKernelCenter) continue;

				float2 SampleOffset = float2(x, y);
				if (KernelConfig.SampleSet == SAMPLE_SET_3X3_SOBEK2018)
				{
					SampleOffset = mul(float2x2(float2(2, -1), float2(1, 2)), SampleOffset);
				}
			
				float2 SampleBufferUV = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
		
				float KernelWeight = 1;
				if (KernelConfig.SampleSet == SAMPLE_SET_5X5_WAVELET)
				{
					KernelWeight = 
						kWaveletFilterWeights5x5[abs(x)] *
						kWaveletFilterWeights5x5[abs(y)] *
						rcp(kWaveletFilterWeights5x5[0] * kWaveletFilterWeights5x5[0]);
				}

				SampleAndAccumulateMultiplexedSignals(
					KernelConfig,
					SignalBuffer0,
					SignalBuffer1,
					SignalBuffer2,
					SignalBuffer3,
					/* inout */ UncompressedAccumulators,
					/* inout */ CompressedAccumulators,
					SampleBufferUV,
					KernelWeight,
					/* bForceSample = */ bIsKernelCenterSample && KernelConfig.bForceKernelCenterAccumulation);
			}
		}
	}
} // AccumulateSquareKernel()

void BroadcastAccumulateSquare3x3KernelCenter(
 	FSSDKernelConfig KernelConfig,
 	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
 	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
 	FSSDSampleSceneInfos RefSceneMetadata,
 	float2 SampleBufferUV,
 	FSSDSampleSceneInfos SampleSceneMetadata,
 	FSSDSignalArray SampleMultiplexedSamples,
	FSSDSignalFrequencyArray SampleMultiplexedFrequencies)
#if CONFIG_ENABLE_WAVE_BROADCAST
{
	const FWaveBroadcastSettings BroadcastSettingsX = InitWaveSwapWithinLaneGroup(/* LaneGroupSize = */ 2);
	const FWaveBroadcastSettings BroadcastSettingsY = InitWaveSwapWithinLaneGroup(/* LaneGroupSize = */ 16);
			
	// Broadcast X.
	SampleBufferUV = WaveBroadcast(BroadcastSettingsX, SampleBufferUV);
	SampleSceneMetadata = WaveBroadcastSceneMetadata(BroadcastSettingsX, SampleSceneMetadata);
	SampleMultiplexedSamples = WaveBroadcastSignalArray(BroadcastSettingsX, SampleMultiplexedSamples);
	SampleMultiplexedFrequencies = WaveBroadcastSignalFrequenciesArray(BroadcastSettingsX, SampleMultiplexedFrequencies);

	if (KernelConfig.SampleSet == SAMPLE_SET_3X3 || KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS)
	{
		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			SampleMultiplexedSamples,
			SampleMultiplexedFrequencies,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ false,
			/* bIsOutsideFrustum = */ false);
	}
		
	// Broadcast Y.
	SampleBufferUV = WaveBroadcast(BroadcastSettingsY, SampleBufferUV);
	SampleSceneMetadata = WaveBroadcastSceneMetadata(BroadcastSettingsY, SampleSceneMetadata);
	SampleMultiplexedSamples = WaveBroadcastSignalArray(BroadcastSettingsY, SampleMultiplexedSamples);
	SampleMultiplexedFrequencies = WaveBroadcastSignalFrequenciesArray(BroadcastSettingsY, SampleMultiplexedFrequencies);

	if (KernelConfig.SampleSet == SAMPLE_SET_3X3 || KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS)
	{
		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			SampleMultiplexedSamples,
			SampleMultiplexedFrequencies,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ false,
			/* bIsOutsideFrustum = */ false);
	}
	
	// Broadcast X Again.
	SampleBufferUV = WaveBroadcast(BroadcastSettingsX, SampleBufferUV);
	SampleSceneMetadata = WaveBroadcastSceneMetadata(BroadcastSettingsX, SampleSceneMetadata);
	SampleMultiplexedSamples = WaveBroadcastSignalArray(BroadcastSettingsX, SampleMultiplexedSamples);
	SampleMultiplexedFrequencies = WaveBroadcastSignalFrequenciesArray(BroadcastSettingsX, SampleMultiplexedFrequencies);
	
	if (KernelConfig.SampleSet == SAMPLE_SET_3X3 || KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS)
	{
		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			SampleMultiplexedSamples,
			SampleMultiplexedFrequencies,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ false,
			/* bIsOutsideFrustum = */ false);
	}
} // BroadcastAccumulateSquare3x3KernelCenter()
#else
{ }
#endif

void AccumulateSquare3x3Kernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
#if CONFIG_ENABLE_WAVE_BROADCAST
{
	if (KernelConfig.bSampleKernelCenter)
	{
		float2 SampleBufferUV = KernelConfig.BufferUV;
			
		// TODO(Denoiser):
		const bool bIsOutsideFrustum = false;

		FSSDCompressedSceneInfos CompressedSampleSceneMetadata;
		FSSDCompressedMultiplexedSample CompressedMultiplexedSamples;
		
		// Force all the signal texture fetch and metadata to overlap to minimize serial texture fetches.
		ISOLATE
		{
			SampleMultiplexedSignals(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				SampleBufferUV,
				/* out */ CompressedSampleSceneMetadata,
				/* out */ CompressedMultiplexedSamples);
		}

		FSSDSampleSceneInfos RefSceneMetadata = UncompressRefSceneMetadata(KernelConfig);

		FSSDSignalArray MultiplexedSamples;
		FSSDSignalFrequencyArray MultiplexedFrequencies;
		UncompressMultiplexedSignals(
			KernelConfig,
			SampleBufferUV,
			CompressedMultiplexedSamples,
			/* out */ MultiplexedSamples,
			/* out */ MultiplexedFrequencies);

		FSSDSampleSceneInfos SampleSceneMetadata = UncompressSampleSceneMetadata(
			KernelConfig, SampleBufferUV, CompressedSampleSceneMetadata);

		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			MultiplexedSamples,
			MultiplexedFrequencies,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ true,
			bIsOutsideFrustum);
		
		BroadcastAccumulateSquare3x3KernelCenter(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			SampleBufferUV,
			SampleSceneMetadata,
			MultiplexedSamples,
			MultiplexedFrequencies);
	}

	// Store whether needs to flip offsets to have lowest VGPR pressure.
	uint2 OutputPixelPostion = BufferUVToBufferPixelCoord(KernelConfig.RefBufferUV);
	bool bFlipX = (OutputPixelPostion.x & 0x1) != 0;
	bool bFlipY = (OutputPixelPostion.y & 0x1) != 0;

	if (KernelConfig.SampleSet == SAMPLE_SET_3X3 || KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS)
	{
		float2 SampleOffset = float2(bFlipX ? 1.0 : -1.0, bFlipY ? 1.0 : -1.0);

		float2 SampleBufferUV = KernelConfig.BufferUV + SampleOffset * KernelConfig.BufferSizeAndInvSize.zw;

		SampleAndAccumulateMultiplexedSignals(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUV,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ false);
	}

	static const float2 SampleOffsetArray[4] = {
		float2(-1.0,  0.0),
		float2( 0.0, -1.0),
		float2(-1.0, 1.0),
		float2( 1.0, -1.0),
	};
		
	UNROLL
	for (
		uint BatchId = (KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS ? 1 : 0);
		BatchId < (KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS ? 1 : 2);
		BatchId++)
	ISOLATE
	{
		
		float2 SampleOffset0 = select(bool2(bFlipX, bFlipY), -SampleOffsetArray[BatchId * 2 + 0], SampleOffsetArray[BatchId * 2 + 0]);
		float2 SampleOffset1 = select(bool2(bFlipX, bFlipY), -SampleOffsetArray[BatchId * 2 + 1], SampleOffsetArray[BatchId * 2 + 1]);
	
		float2 SampleBufferUV[2];
		SampleBufferUV[0] = KernelConfig.BufferUV + SampleOffset0 * KernelConfig.BufferSizeAndInvSize.zw;
		SampleBufferUV[1] = KernelConfig.BufferUV + SampleOffset1 * KernelConfig.BufferSizeAndInvSize.zw;

		SampleAndAccumulateMultiplexedSignalsPair(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUV,
			/* KernelWeight = */ 1.0);
	}
} // AccumulateSquare3x3Kernel()
#else // !CONFIG_ENABLE_WAVE_BROADCAST
{
	if (KernelConfig.bSampleKernelCenter)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}

	static const float2 SampleOffsetArray[4] = {
		float2(1.0, 0.0),
		float2(1.0, 1.0),
		float2(0.0, 1.0),
		float2(-1.0, 1.0),
	};

	UNROLL
	for (
		uint BatchId = (KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS ? 1 : 0);
		BatchId < 4;
		BatchId += (KernelConfig.SampleSet != SAMPLE_SET_3X3 ? 2 : 1))
	ISOLATE
	{
		float2 SampleOffset = SampleOffsetArray[BatchId];
	
		float2 SampleBufferUV[2];
		SampleBufferUV[0] = KernelConfig.BufferUV + SampleOffset * KernelConfig.BufferSizeAndInvSize.zw;
		SampleBufferUV[1] = KernelConfig.BufferUV - SampleOffset * KernelConfig.BufferSizeAndInvSize.zw;

		SampleAndAccumulateMultiplexedSignalsPair(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUV,
			/* KernelWeight = */ 1.0);
	}
} // AccumulateSquare3x3Kernel()
#endif // !CONFIG_ENABLE_WAVE_BROADCAST

#endif // COMPILE_BOX_KERNEL


//------------------------------------------------------- STACKOWIAK 2018

#if COMPILE_STACKOWIAK_KERNEL

static const float2 kStackowiakSampleSet0[56 * 4] =
{
	float2(-0.5, -0.5), float2(+0.5, -0.5), float2(-0.5, +0.5), float2(+0.5, +0.5),
	float2(-1.5, +0.5), float2(-1.5, -0.5), float2(-0.5, +1.5), float2(+1.5, -0.5),
	float2(+0.5, -1.5), float2(+2.5, -0.5), float2(+1.5, +0.5), float2(-0.5, -1.5),
	float2(-1.5, -2.5), float2(-0.5, -2.5), float2(-1.5, -1.5), float2(-0.5, +2.5),
	float2(-1.5, +1.5), float2(+1.5, -2.5), float2(-1.5, +2.5), float2(+1.5, +2.5),
	float2(+0.5, -2.5), float2(-2.5, -0.5), float2(-2.5, -1.5), float2(-2.5, +0.5),
	float2(+0.5, +1.5), float2(+0.5, +2.5), float2(-3.5, +0.5), float2(+0.5, +3.5),
	float2(+1.5, -1.5), float2(+3.5, -0.5), float2(+2.5, +1.5), float2(+3.5, +0.5),
	float2(+1.5, +1.5), float2(-2.5, +1.5), float2(-3.5, +2.5), float2(+3.5, +1.5),
	float2(-3.5, -0.5), float2(-1.5, -3.5), float2(-2.5, -2.5), float2(-2.5, +2.5),
	float2(+2.5, +0.5), float2(+2.5, +2.5), float2(+1.5, +3.5), float2(+3.5, -1.5),
	float2(-3.5, -2.5), float2(+3.5, -2.5), float2(+2.5, -1.5), float2(+0.5, -3.5),
	float2(-0.5, +3.5), float2(-0.5, -4.5), float2(-4.5, +0.5), float2(+4.5, +0.5),
	float2(-4.5, -1.5), float2(-3.5, +1.5), float2(-0.5, -3.5), float2(+1.5, -3.5),
	float2(+0.5, -4.5), float2(-1.5, +3.5), float2(+0.5, +4.5), float2(-3.5, -1.5),
	float2(-4.5, +1.5), float2(+2.5, -4.5), float2(+2.5, -2.5), float2(-1.5, +4.5),
	float2(-2.5, -4.5), float2(+4.5, -2.5), float2(+2.5, +3.5), float2(-3.5, +3.5),
	float2(-2.5, +3.5), float2(+0.5, -5.5), float2(-4.5, +3.5), float2(-2.5, -3.5),
	float2(-4.5, +2.5), float2(+3.5, +3.5), float2(+2.5, -3.5), float2(+4.5, +3.5),
	float2(+3.5, -3.5), float2(+4.5, +2.5), float2(-5.5, +1.5), float2(-4.5, -0.5),
	float2(+3.5, +2.5), float2(-0.5, +4.5), float2(-1.5, +5.5), float2(+1.5, +5.5),
	float2(+4.5, -0.5), float2(+5.5, +0.5), float2(+4.5, +1.5), float2(-1.5, -4.5),
	float2(-1.5, -5.5), float2(-4.5, -2.5), float2(-2.5, +5.5), float2(+2.5, +5.5),
	float2(+1.5, +4.5), float2(+5.5, +1.5), float2(+1.5, -4.5), float2(-3.5, -3.5),
	float2(+3.5, -4.5), float2(-3.5, -4.5), float2(+4.5, -1.5), float2(+4.5, -3.5),
	float2(-3.5, -5.5), float2(-2.5, -5.5), float2(-4.5, -3.5), float2(+4.5, +4.5),
	float2(-3.5, +4.5), float2(-2.5, +4.5), float2(-5.5, -2.5), float2(-5.5, +0.5),
	float2(+2.5, -5.5), float2(+3.5, +4.5), float2(-0.5, -5.5), float2(-0.5, +6.5),
	float2(+2.5, +4.5), float2(-5.5, -0.5), float2(-6.5, -1.5), float2(+1.5, -5.5),
	float2(-6.5, -0.5), float2(+0.5, +5.5), float2(+1.5, +6.5), float2(+6.5, +1.5),
	float2(-0.5, +5.5), float2(+6.5, -0.5), float2(-4.5, -4.5), float2(-5.5, +2.5),
	float2(+5.5, -0.5), float2(-5.5, -1.5), float2(-6.5, +3.5), float2(-1.5, +6.5),
	float2(-6.5, +0.5), float2(+4.5, -5.5), float2(-3.5, +6.5), float2(+6.5, -1.5),
	float2(+0.5, -6.5), float2(-5.5, -3.5), float2(+5.5, -2.5), float2(+4.5, -4.5),
	float2(+5.5, -1.5), float2(+3.5, -6.5), float2(+5.5, +3.5), float2(+3.5, -5.5),
	float2(-5.5, -4.5), float2(+6.5, -3.5), float2(-0.5, -6.5), float2(+3.5, +6.5),
	float2(-5.5, +3.5), float2(+0.5, +6.5), float2(+6.5, +0.5), float2(+6.5, -2.5),
	float2(-6.5, -3.5), float2(-4.5, +4.5), float2(-7.5, -0.5), float2(+7.5, +0.5),
	float2(+5.5, +2.5), float2(-0.5, -7.5), float2(+0.5, +7.5), float2(-4.5, +5.5),
	float2(+3.5, +5.5), float2(-3.5, +5.5), float2(-4.5, -5.5), float2(+4.5, +6.5),
	float2(+5.5, -4.5), float2(+4.5, +5.5), float2(-4.5, +6.5), float2(+6.5, +4.5),
	float2(-7.5, +1.5), float2(-6.5, +1.5), float2(+5.5, -3.5), float2(-6.5, +2.5),
	float2(-2.5, +6.5), float2(-1.5, -7.5), float2(+5.5, +4.5), float2(-1.5, -6.5),
	float2(-3.5, -7.5), float2(+2.5, -7.5), float2(-7.5, +2.5), float2(-6.5, -2.5),
	float2(-5.5, +5.5), float2(+2.5, +6.5), float2(-2.5, -6.5), float2(-7.5, +0.5),
	float2(-0.5, +7.5), float2(+7.5, -2.5), float2(-2.5, +7.5), float2(+0.5, -7.5),
	float2(-4.5, -7.5), float2(+7.5, +1.5), float2(+1.5, -6.5), float2(-6.5, +4.5),
	float2(-1.5, +7.5), float2(-5.5, -5.5), float2(+6.5, +2.5), float2(-3.5, -6.5),
	float2(+3.5, -7.5), float2(-5.5, +4.5), float2(+2.5, -6.5), float2(+1.5, -7.5),
	float2(+6.5, +3.5), float2(+5.5, -6.5), float2(-6.5, +5.5), float2(+7.5, +4.5),
	float2(+7.5, -1.5), float2(-7.5, -1.5), float2(+3.5, +7.5), float2(-5.5, +6.5),
	float2(+1.5, +7.5), float2(+7.5, +3.5), float2(+7.5, -0.5), float2(-7.5, -2.5),
	float2(+5.5, +5.5), float2(+6.5, +5.5), float2(+5.5, -5.5), float2(-2.5, -7.5),
	float2(+2.5, +7.5), float2(-7.5, -3.5), float2(-7.5, -4.5), float2(-6.5, -4.5),
	float2(+7.5, -3.5), float2(+5.5, +6.5), float2(-5.5, -6.5), float2(-4.5, -6.5),
	float2(+7.5, +2.5), float2(-7.5, +3.5), float2(+4.5, -6.5), float2(+7.5, -4.5),
};

static const float2 kStackowiakSampleSet1[56 * 4] =
{
    float2(-0.5, -0.5), float2(+0.5, -0.5), float2(-0.5, +0.5), float2(+0.5, +0.5),
    float2(+0.5, -1.5), float2(+1.5, -1.5), float2(-1.5, -0.5), float2(+1.5, +1.5),
    float2(-0.5, -2.5), float2(-1.5, -1.5), float2(+0.5, +1.5), float2(-1.5, +0.5),
    float2(+1.5, -0.5), float2(-0.5, +1.5), float2(-2.5, +0.5), float2(+0.5, +2.5),
    float2(-2.5, -1.5), float2(+2.5, +0.5), float2(+1.5, +0.5), float2(-0.5, -1.5),
    float2(-1.5, +1.5), float2(+2.5, -2.5), float2(-3.5, -0.5), float2(-1.5, +2.5),
    float2(-2.5, +1.5), float2(-2.5, -0.5), float2(-1.5, -2.5), float2(+2.5, -1.5),
    float2(-3.5, +0.5), float2(-0.5, -3.5), float2(-1.5, +3.5), float2(+0.5, -2.5),
    float2(+1.5, +2.5), float2(-0.5, +2.5), float2(+0.5, +3.5), float2(+3.5, +0.5),
    float2(+2.5, +1.5), float2(-2.5, -2.5), float2(+2.5, -0.5), float2(+3.5, -1.5),
    float2(-0.5, +3.5), float2(+3.5, +1.5), float2(-3.5, +2.5), float2(+3.5, +2.5),
    float2(+3.5, -0.5), float2(+0.5, -4.5), float2(-2.5, +3.5), float2(+0.5, -3.5),
    float2(-1.5, -4.5), float2(+1.5, +3.5), float2(+1.5, -2.5), float2(-3.5, +1.5),
    float2(+2.5, -3.5), float2(-2.5, -3.5), float2(+2.5, +2.5), float2(+1.5, +4.5),
    float2(-4.5, -2.5), float2(-2.5, +2.5), float2(-4.5, +1.5), float2(+4.5, +1.5),
    float2(-2.5, -4.5), float2(+3.5, -3.5), float2(-1.5, -3.5), float2(-3.5, -1.5),
    float2(+1.5, -4.5), float2(+4.5, -2.5), float2(+1.5, -3.5), float2(-1.5, +4.5),
    float2(-4.5, +2.5), float2(-4.5, -0.5), float2(+2.5, +4.5), float2(-4.5, +0.5),
    float2(-3.5, -4.5), float2(+0.5, +4.5), float2(+3.5, -2.5), float2(-3.5, -2.5),
    float2(-3.5, +3.5), float2(+3.5, +3.5), float2(+4.5, +0.5), float2(+0.5, +5.5),
    float2(-0.5, +4.5), float2(+4.5, -3.5), float2(-1.5, +5.5), float2(-0.5, -4.5),
    float2(+2.5, +3.5), float2(+4.5, +2.5), float2(-2.5, +5.5), float2(+2.5, -4.5),
    float2(+4.5, -0.5), float2(+5.5, -0.5), float2(-4.5, +4.5), float2(+5.5, -1.5),
    float2(-5.5, -1.5), float2(-4.5, -1.5), float2(+3.5, +4.5), float2(-3.5, -3.5),
    float2(-5.5, +0.5), float2(+1.5, -5.5), float2(-5.5, -2.5), float2(-3.5, +4.5),
    float2(+0.5, -5.5), float2(-2.5, -5.5), float2(+2.5, +5.5), float2(+4.5, +4.5),
    float2(+4.5, -1.5), float2(-2.5, +4.5), float2(+4.5, +3.5), float2(+0.5, +6.5),
    float2(-0.5, -6.5), float2(+5.5, +2.5), float2(-0.5, -5.5), float2(-5.5, -0.5),
    float2(-6.5, -1.5), float2(-0.5, +5.5), float2(-0.5, +6.5), float2(+6.5, -0.5),
    float2(+1.5, +5.5), float2(+1.5, -6.5), float2(+5.5, +0.5), float2(-5.5, +2.5),
    float2(+5.5, +1.5), float2(-5.5, +1.5), float2(-6.5, -0.5), float2(-1.5, -5.5),
    float2(-5.5, -4.5), float2(-4.5, +3.5), float2(-6.5, +1.5), float2(+2.5, -5.5),
    float2(+3.5, -5.5), float2(-5.5, -3.5), float2(+1.5, +6.5), float2(+6.5, +2.5),
    float2(+4.5, -4.5), float2(+3.5, -6.5), float2(-4.5, -4.5), float2(-4.5, -3.5),
    float2(-6.5, +2.5), float2(+3.5, +5.5), float2(+3.5, -4.5), float2(+5.5, -3.5),
    float2(-5.5, +4.5), float2(+6.5, -3.5), float2(-6.5, -2.5), float2(+5.5, +4.5),
    float2(-1.5, +6.5), float2(-0.5, -7.5), float2(-6.5, +3.5), float2(-5.5, +3.5),
    float2(-6.5, -4.5), float2(+7.5, -1.5), float2(-3.5, -5.5), float2(+3.5, +6.5),
    float2(+5.5, +3.5), float2(+7.5, +0.5), float2(+5.5, -2.5), float2(-6.5, +0.5),
    float2(-7.5, +1.5), float2(-3.5, -6.5), float2(+6.5, +0.5), float2(+7.5, +1.5),
    float2(-2.5, -7.5), float2(-3.5, +5.5), float2(-7.5, -0.5), float2(-3.5, +6.5),
    float2(-2.5, +6.5), float2(+4.5, -6.5), float2(-5.5, +5.5), float2(+4.5, -5.5),
    float2(+6.5, -2.5), float2(+6.5, +3.5), float2(-1.5, -6.5), float2(-1.5, +7.5),
    float2(+6.5, +1.5), float2(-5.5, -5.5), float2(+0.5, -6.5), float2(+7.5, +3.5),
    float2(+2.5, +6.5), float2(-4.5, +5.5), float2(-6.5, -3.5), float2(-4.5, -5.5),
    float2(-6.5, -5.5), float2(+5.5, -6.5), float2(-2.5, -6.5), float2(+5.5, -5.5),
    float2(+4.5, +5.5), float2(-7.5, +0.5), float2(+6.5, -1.5), float2(+0.5, -7.5),
    float2(+7.5, -0.5), float2(-3.5, -7.5), float2(+2.5, -6.5), float2(-3.5, +7.5),
    float2(-4.5, -7.5), float2(-0.5, +7.5), float2(-6.5, +5.5), float2(+7.5, -3.5),
    float2(-4.5, +6.5), float2(+1.5, +7.5), float2(+5.5, -4.5), float2(+7.5, +4.5),
    float2(+0.5, +7.5), float2(+4.5, +6.5), float2(-4.5, +7.5), float2(-7.5, -1.5),
    float2(+3.5, -7.5), float2(+7.5, -4.5), float2(+3.5, +7.5), float2(-1.5, -7.5),
    float2(+6.5, -4.5), float2(-7.5, -3.5), float2(+6.5, +4.5), float2(+2.5, -7.5),
    float2(+7.5, -2.5), float2(-7.5, +2.5), float2(+1.5, -7.5), float2(-5.5, +6.5),
    float2(+5.5, +5.5), float2(-2.5, +7.5), float2(+7.5, +2.5), float2(-7.5, -2.5),
    float2(+2.5, +7.5), float2(-6.5, +4.5), float2(+5.5, +6.5), float2(-4.5, -6.5),
};

static const uint kStackowiakSampleSetCount = 4;
static const uint kStackowiakSampleCountPerSet = 56;

void ConvolveStackowiakKernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	// Number of batch size done at same time, to improve lattency hidding.
	const uint kSamplingBatchSize = 2;

	if (KernelConfig.bDescOrder)
	{
		// (SALU) Number of batch of samples to perform.
		const uint BatchCountCount = (KernelConfig.SampleCount + (kSamplingBatchSize - 1)) / kSamplingBatchSize;

		// (SALU) Compute a final number of sample quantize the sampling batch size.
		const uint SampleCount = BatchCountCount * kSamplingBatchSize;

		// Compile time number of samples between rings.
		const uint StocasticSamplesPerCluster = 8 / kStackowiakSampleSetCount;

		// Compute the first index at witch digestion must happen.
		uint CurrentRingId = 0;
		uint NextClusterBoundary = 0;

		if (StocasticSamplesPerCluster == 2)
		{
			uint un = SampleCount - 1;

			CurrentRingId = (uint(floor(sqrt(4 * un - 3))) + 1) / 2;

			NextClusterBoundary = 1 + CurrentRingId * (CurrentRingId - 1);
		}
		else
		{
			// TODO(Denoiser)
		}
		
		FSSDSampleClusterInfo ClusterInfo;
		ClusterInfo.OutterBoundaryRadius = (CurrentRingId + 1) * KernelConfig.KernelSpreadFactor;

		StartAccumulatingCluster(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			ClusterInfo);
		
		// Processes the samples in batches so that the compiler can do lattency hidding.
		LOOP
		for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
		{
			UNROLL_N(2)
			for (uint SampleBatchId = 0; SampleBatchId < 2; SampleBatchId++)
			{
				uint SampleId = (BatchCountCount - BatchId) * kSamplingBatchSize - 1 - SampleBatchId;

				bool bIsKernelCenterSample = SampleId == 0 && (SampleBatchId == (kSamplingBatchSize - 1));
				
				uint SampleTrackId = KernelConfig.SampleTrackId;

				float2 SampleOffset = kStackowiakSampleSet0[kStackowiakSampleSetCount * SampleId + SampleTrackId];
				if (KernelConfig.SampleSubSetId == 1)
				{
					SampleOffset = kStackowiakSampleSet1[kStackowiakSampleSetCount * SampleId + SampleTrackId];
				}
				
				float2 SampleBufferUV = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
				
				float KernelWeight = 1;
				SampleAndAccumulateMultiplexedSignals(
					KernelConfig,
					SignalBuffer0,
					SignalBuffer1,
					SignalBuffer2,
					SignalBuffer3,
					/* inout */ UncompressedAccumulators,
					/* inout */ CompressedAccumulators,
					SampleBufferUV,
					/* KernelWeight = */ 1.0,
					/* bForceSample = */ bIsKernelCenterSample && KernelConfig.bForceKernelCenterAccumulation);

				// Change of cluster. Can only happens on odd SampleId, meaning even SampleBatchId.
				BRANCH
				if (SampleId == NextClusterBoundary && (SampleBatchId % 2) == 0)
				{
					// Compute the number samples that have been accumulated for this sample.
					uint SampleCountForCluster = min(CurrentRingId * StocasticSamplesPerCluster, SampleCount - SampleId);

					// Digest all acumulators.
					DijestAccumulatedClusterSamples(
						/* inout */ UncompressedAccumulators,
						/* inout */ CompressedAccumulators,
						CurrentRingId, SampleCountForCluster);
				
					BRANCH
					if (!KernelConfig.bSampleKernelCenter && SampleId == 1)
					{
						break;
					}

					// Change cluster index and boundary.
					CurrentRingId -= 1;
					NextClusterBoundary -= CurrentRingId * StocasticSamplesPerCluster;
				
					FSSDSampleClusterInfo ClusterInfo;
					ClusterInfo.OutterBoundaryRadius = (CurrentRingId + 1) * KernelConfig.KernelSpreadFactor;

					// Prepare the accumulators for new cluster.
					StartAccumulatingCluster(
						KernelConfig, 
						/* inout */ UncompressedAccumulators,
						/* inout */ CompressedAccumulators,
						ClusterInfo);
				}
			} // for (uint SampleBatchId = 0; SampleBatchId < kSamplingBatchSize; SampleBatchId++)
		} // for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
		
		// NextClusterBoundary is not capable to reach 0, therefore need to manually digest the center sample.
		if (KernelConfig.bSampleKernelCenter)
		{
			DijestAccumulatedClusterSamples(
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				/* RingId = */ 0, /* SampleCount = */ 1);
		}
	}
	else // if (!KernelConfig.bDescOrder)
	{
		if (KernelConfig.bSampleKernelCenter)
		{
			SampleAndAccumulateCenterSampleAsItsOwnCluster(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators);
		}

		// Accumulate second sample to lattency hide with the center sample.
		{
			uint SampleTrackId = KernelConfig.SampleTrackId;

			uint SampleId = 1;

			float2 SampleOffset = kStackowiakSampleSet0[kStackowiakSampleSetCount * SampleId + SampleTrackId];
			if (KernelConfig.SampleSubSetId == 1)
			{
				SampleOffset = kStackowiakSampleSet1[kStackowiakSampleSetCount * SampleId + SampleTrackId];
			}
			
			float2 SampleBufferUV = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
			
			SampleAndAccumulateMultiplexedSignals(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				SampleBufferUV,
				/* KernelWeight = */ 1.0,
				/* bForceSample = */ false);
		}
		
		// (SALU) Number of batch of samples to perform.
		const uint BatchCountCount = (KernelConfig.SampleCount - 1) / kSamplingBatchSize;

		// Processes the samples in batches so that the compiler can do lattency hidding.
		// TODO(Denoiser): kSamplingBatchSize for lattency hidding
		LOOP
		for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
		{
			float2 SampleBufferUV[2];
			
			uint SampleTrackId = KernelConfig.SampleTrackId;

			UNROLL_N(2)
			for (uint SampleBatchId = 0; SampleBatchId < 2; SampleBatchId++)
			{
				uint SampleId = BatchId * kSamplingBatchSize + (SampleBatchId + kSamplingBatchSize);

				float2 SampleOffset = kStackowiakSampleSet0[kStackowiakSampleSetCount * SampleId + SampleTrackId];
				if (KernelConfig.SampleSubSetId == 1)
				{
					SampleOffset = kStackowiakSampleSet1[kStackowiakSampleSetCount * SampleId + SampleTrackId];
				}
			
				SampleBufferUV[SampleBatchId] = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
			}

			SampleAndAccumulateMultiplexedSignalsPair(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				SampleBufferUV,
				/* KernelWeight = */ 1.0);
		} // for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
	} // if (!KernelConfig.bDescOrder)
} // ConvolveStackowiakKernel()

#endif // COMPILE_STACKOWIAK_KERNEL


//------------------------------------------------------- DISK

#if COMPILE_DISK_KERNEL

// Returns the position of the sample on the unit circle (radius = 1) for a given ring.
float2 GetDiskSampleOnUnitCirle(uint RingId, uint RingSampleIteration, uint RingSampleId)
{
	RingId -= 1; // TODO(Denoiser).

	float SampleRingPos = RingSampleId;

	// Do not allign all j == 0 samples of the different ring on the X axis to increase minimal distance between all
	// samples, that reduce variance to clean by post filtering.
	#if 1
		SampleRingPos += (RingId - 2 * (RingId / 2)) * 0.5;
	#endif

	#if 1
		SampleRingPos += (RingId + 1) * 0.2;
	#endif

	float SampleAngle = PI * SampleRingPos / float(RingSampleIteration);
			
	return float2(cos(SampleAngle), sin(SampleAngle));
}

// Returns the rotation matrix to use between sample of the ring.
float2x2 GetSampleRotationMatrix(uint RingSampleIteration)
{
	float RotationAngle = PI / float(RingSampleIteration);

	float C = cos(RotationAngle);
	float S = sin(RotationAngle);

	return float2x2(
		float2( C,  S),
		float2(-S,  C));
}

// Returns the total number of sampling iteration for a given ring id.
uint GetRingSamplingPairCount(const uint SampleSet, uint RingId)
{
	if (SampleSet == SAMPLE_SET_HEXAWEB)
	{
		return RingId * 3;
	}

	// This number of sample is carefully chosen to have exact number of sample a square shaped ring (SquarePos).
	return RingId * 4;
}

// Returns the total number of sample of the kernel.
uint GetDiskKernelSampleCount(const uint SampleSet, uint RingCount)
{
	if (SampleSet == SAMPLE_SET_HEXAWEB)
	{
		return 1 + 3 * RingCount * (RingCount + 1);
	}

	// Depends on GetRingSamplingPairCount().
	return 1 + 4 * RingCount * (RingCount + 1);
}

// Transform at compile time a 2 dimensional batch's constant into sample pair constant, by using rotation invariance.
float2 SampleConstFromBatchConst(const uint BatchSampleId, float2 BatchConst)
{
	/**
	 *             Y
	 *             ^
	 *             |
	 *        1    |
	 *             |
	 *             |       0
	 *             |
	 * - - - - - - O - - - - > X
	 */
	if (BatchSampleId == 1)
		return float2(-BatchConst.y, BatchConst.x);
	return BatchConst;
}



// Gather a ring into the accumulator.
void GatherRingSamples(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	const uint RingId)
{
	// Number of sample iteration for this ring.
	const uint RingSamplePairCount = GetRingSamplingPairCount(KernelConfig.SampleSet, RingId);

	// Number of sample pair to process per batch.
	// TODO(Denoiser): Could potentially do 4 using symetries? Might be unpracticable because of VGPR pressure. 
	const uint SamplePairBatchSize = (KernelConfig.SampleSet == SAMPLE_SET_HEXAWEB) ? 1 : 2;

	// Number of batch to process.
	const uint BatchCount = RingSamplePairCount / SamplePairBatchSize;

	// Distance of the ring from the center of the kernel in sample count.
	const uint RingDistance = uint(RingId + 0);
	
	// Generate at compile time sample rotation matrix.
	const float2x2 SampleRotationMatrix = GetSampleRotationMatrix(RingSamplePairCount);

	// Generates at compile time first sample location on circle (radius = 1).
	const float2 FirstCircleUnitPos = GetDiskSampleOnUnitCirle(RingId, RingSamplePairCount, /* BatchId = */ 0);

	// Position of the first sample on circle with radius according to KernelRadius.
	float2 FirstCircleSamplePosOffset = (RingDistance * FirstCircleUnitPos) * KernelConfig.KernelSpreadFactor;

	// Setup iteratable SGPR
	float2 CurrentCircleUnitPos = FirstCircleUnitPos;
	float2 CurrentCircleSamplePosOffset = FirstCircleSamplePosOffset;

	#if CONFIG_SGPR_HINT_OPTIMIZATION
	{
		CurrentCircleUnitPos = ToScalarMemory(CurrentCircleUnitPos);
		CurrentCircleSamplePosOffset = ToScalarMemory(CurrentCircleSamplePosOffset);
	}
	#endif

	// Loops through all batch of samples to process.
	LOOP
	for (uint BatchId = 0; BatchId < BatchCount; BatchId++)
	{
		// Rotate the samples position along the ring.
		CurrentCircleUnitPos = mul(CurrentCircleUnitPos, SampleRotationMatrix);
		CurrentCircleSamplePosOffset = mul(CurrentCircleSamplePosOffset, SampleRotationMatrix);

		#if CONFIG_SGPR_HINT_OPTIMIZATION
		{
			CurrentCircleUnitPos = ToScalarMemory(CurrentCircleUnitPos);
			CurrentCircleSamplePosOffset = ToScalarMemory(CurrentCircleSamplePosOffset);
		}
		#endif

		// Sample in batch of multiple pair to increase texture fetch concurency, to have better
		// lattency hidding.
		UNROLL
		for (uint BatchSampleId = 0; BatchSampleId < SamplePairBatchSize; BatchSampleId++)
		{
			float2 CircleSamplePosOffset = SampleConstFromBatchConst(BatchSampleId, CurrentCircleSamplePosOffset);
			
			float2 SampleUVPair[2];
			SampleUVPair[0] = KernelConfig.BufferUV + CircleSamplePosOffset * KernelConfig.BufferSizeAndInvSize.zw;
			SampleUVPair[1] = KernelConfig.BufferUV - CircleSamplePosOffset * KernelConfig.BufferSizeAndInvSize.zw;

			SampleAndAccumulateMultiplexedSignalsPair(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				SampleUVPair,
				/* KernelWeight = */ 1.0);
		} // for (uint BatchSampleId = 0; BatchSampleId < SamplePairBatchSize; BatchSampleId++)
	} // for (uint BatchId = 0; BatchId < BatchCount; BatchId++)
}

void ConvolveDiskKernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	// Accumulate the center of the kernel.
	if (KernelConfig.bSampleKernelCenter && !KernelConfig.bDescOrder)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}

	// Accumulate each ring. Use LOOP, because FXC is going through its pace otherwise.
	#if 1
		LOOP
	#else
		UNROLL
	#endif
	for (
		uint RingId = (KernelConfig.bDescOrder ? KernelConfig.RingCount : 1);
		(KernelConfig.bDescOrder ? RingId > 0 : RingId <= KernelConfig.RingCount);
		RingId += (KernelConfig.bDescOrder ? ~0u : 1))
	{
		const uint RingSamplePairCount = GetRingSamplingPairCount(KernelConfig.SampleSet, RingId);
		
		FSSDSampleClusterInfo ClusterInfo;
		ClusterInfo.OutterBoundaryRadius = (RingId + 1) * KernelConfig.KernelSpreadFactor;

		StartAccumulatingCluster(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			ClusterInfo);
		
		GatherRingSamples(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RingId);
			
		DijestAccumulatedClusterSamples(
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RingId, RingSamplePairCount * 2);
	} // for (uint RingId = 0; RingId < KernelConfig.RingCount; RingId++)
		
	// Accumulate the center of the kernel.
	if (KernelConfig.bSampleKernelCenter && KernelConfig.bDescOrder)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
}

#endif // COMPILE_DISK_KERNEL


//------------------------------------------------------- DIRECTIONAL KERNELS

#if COMPILE_DIRECTIONAL_KERNEL

void ConvolveDirectionalRect(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	// Accumulate the center of the kernel.
	if (KernelConfig.bSampleKernelCenter)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}

	// Number of batch size done at same time, to improve lattency hidding.
	const uint kSamplingBatchSize = 2;

	// Number of batch of samples to perform. It's not round up because also sampling the center of kernel anyway.
	// TODO(Denoiser): store in a SGPR array instead to save 1 VGPR.
	const uint BatchCountCount = KernelConfig.SampleCount / kSamplingBatchSize;

	// Processes the samples in batches so that the compiler can do lattency hidding.
	LOOP
	for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
	{
		float2 SampleBufferUV[2];
			
		UNROLL_N(2)
		for (uint SampleBatchId = 0; SampleBatchId < 2; SampleBatchId++)
		{
			uint SampleId = BatchId * kSamplingBatchSize + SampleBatchId;

			float2 E = Hammersley16(SampleId, BatchCountCount * kSamplingBatchSize, KernelConfig.HammersleySeed);

			if (KernelConfig.SampleSet == SAMPLE_SET_DIRECTIONAL_RECT)
			{
				E = (E * 2.0 - 1.0);
			}
			else // if (KernelConfig.SampleSet == SAMPLE_SET_DIRECTIONAL_ELLIPSE)
			{
				E = UniformSampleDiskConcentric(E);
			}

			float2 SampleOffset = 
				float2(KernelConfig.MajorAxis) * KernelConfig.MajorPixelRadius * E.x +
				float2(-KernelConfig.MajorAxis.y, KernelConfig.MajorAxis.x) * KernelConfig.MinorPixelRadius * E.y;
			
			SampleBufferUV[SampleBatchId] = KernelConfig.BufferUV + SampleOffset * KernelConfig.BufferSizeAndInvSize.zw;
		}

		SampleAndAccumulateMultiplexedSignalsPair(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUV,
			/* KernelWeight = */ 1.0);
	} // for (uint BatchId = 0; BatchId < BatchCountCount; BatchId++)
}

#endif // COMPILE_DIRECTIONAL_KERNEL

//------------------------------------------------------- RAW EXPERIMENTAL KERNEL TO TRY

#if COMPILE_RAW_EXPERIMENTAL_KERNEL

#if 0
static const float2 SampleArray4x4x8[128] = {
	float2(0.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(2.000000, 1.000000),
	float2(-1.000000, 2.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, 2.000000),
	float2(1.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(-1.000000, 2.000000),
	float2(-1.000000, -1.000000),
	float2(0.000000, 1.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(1.000000, 2.000000),
	float2(0.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(0.000000, -1.000000),
	float2(-1.000000, -1.000000),
	float2(2.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(0.000000, 2.000000),
	float2(1.000000, -1.000000),
	float2(1.000000, 2.000000),
	float2(-1.000000, 2.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(2.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, -1.000000),
	float2(-3.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, -1.000000),
	float2(0.000000, 1.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(2.000000, 1.000000),
	float2(1.000000, 0.000000),
	float2(2.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, 2.000000),
	float2(2.000000, 2.000000),
	float2(1.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-2.000000, 1.000000),
	float2(1.000000, 2.000000),
	float2(-3.000000, 0.000000),
	float2(-2.000000, 2.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, -2.000000),
	float2(-1.000000, 0.000000),
	float2(1.000000, -2.000000),
	float2(1.000000, 0.000000),
	float2(2.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(-2.000000, 2.000000),
	float2(-1.000000, 2.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, 3.000000),
	float2(0.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(1.000000, -2.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, 2.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(-2.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(-2.000000, 2.000000),
	float2(1.000000, -2.000000),
	float2(-3.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(2.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(-2.000000, -2.000000),
	float2(2.000000, 1.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(-2.000000, -2.000000),
	float2(-1.000000, 2.000000),
	float2(1.000000, 2.000000),
	float2(2.000000, 0.000000),
	float2(1.000000, 1.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, -1.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(-2.000000, 0.000000),
	float2(2.000000, -1.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, -2.000000),
	float2(-1.000000, -2.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, -2.000000),
	float2(-1.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(-1.000000, 1.000000),
}; // SampleArray4x4x8

#else

static const float2 SampleArray4x4x16[256] = {
	float2(0.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, -1.000000),
	float2(1.000000, 0.000000),
	float2(-2.000000, 1.000000),
	float2(2.000000, -1.000000),
	float2(1.000000, 1.000000),
	float2(-1.000000, -2.000000),
	float2(2.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(2.000000, 2.000000),
	float2(2.000000, -2.000000),
	float2(1.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(2.000000, 3.000000),
	float2(-1.000000, -2.000000),
	float2(1.000000, -2.000000),
	float2(0.000000, -1.000000),
	float2(1.000000, -1.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(-2.000000, -2.000000),
	float2(-2.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(2.000000, -1.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, 3.000000),
	float2(-1.000000, -2.000000),
	float2(0.000000, -1.000000),
	float2(1.000000, -2.000000),
	float2(-1.000000, 0.000000),
	float2(-2.000000, -1.000000),
	float2(2.000000, 2.000000),
	float2(-2.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(1.000000, 1.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(2.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, -1.000000),
	float2(0.000000, 1.000000),
	float2(-2.000000, 1.000000),
	float2(1.000000, -2.000000),
	float2(-3.000000, 0.000000),
	float2(0.000000, -2.000000),
	float2(-2.000000, 2.000000),
	float2(-1.000000, 2.000000),
	float2(1.000000, -1.000000),
	float2(2.000000, 3.000000),
	float2(-2.000000, 0.000000),
	float2(0.000000, 2.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 0.000000),
	float2(2.000000, -1.000000),
	float2(-1.000000, -1.000000),
	float2(-1.000000, 2.000000),
	float2(1.000000, -1.000000),
	float2(-3.000000, 0.000000),
	float2(1.000000, 2.000000),
	float2(2.000000, -2.000000),
	float2(0.000000, -1.000000),
	float2(0.000000, 1.000000),
	float2(0.000000, -2.000000),
	float2(1.000000, 1.000000),
	float2(2.000000, 0.000000),
	float2(2.000000, 1.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, -2.000000),
	float2(2.000000, -1.000000),
	float2(-2.000000, 2.000000),
	float2(-1.000000, 0.000000),
	float2(2.000000, 2.000000),
	float2(3.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, 2.000000),
	float2(0.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(1.000000, 1.000000),
	float2(1.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(2.000000, 0.000000),
	float2(1.000000, 2.000000),
	float2(-1.000000, 2.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, 2.000000),
	float2(3.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(-2.000000, -3.000000),
	float2(2.000000, 3.000000),
	float2(1.000000, 0.000000),
	float2(2.000000, 2.000000),
	float2(1.000000, 1.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, -1.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, 2.000000),
	float2(-1.000000, 0.000000),
	float2(-3.000000, 0.000000),
	float2(-2.000000, -1.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, -1.000000),
	float2(0.000000, 2.000000),
	float2(1.000000, 1.000000),
	float2(0.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(-2.000000, 2.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(2.000000, -1.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, 2.000000),
	float2(-1.000000, -2.000000),
	float2(0.000000, 1.000000),
	float2(1.000000, 2.000000),
	float2(-1.000000, 1.000000),
	float2(2.000000, 2.000000),
	float2(-3.000000, 0.000000),
	float2(-2.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(-1.000000, 0.000000),
	float2(-4.000000, -1.000000),
	float2(2.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, 2.000000),
	float2(0.000000, 1.000000),
	float2(-1.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(-1.000000, 2.000000),
	float2(0.000000, 3.000000),
	float2(2.000000, -2.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 2.000000),
	float2(1.000000, -1.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, 1.000000),
	float2(-2.000000, -1.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, -1.000000),
	float2(-1.000000, -2.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 1.000000),
	float2(1.000000, 2.000000),
	float2(2.000000, -1.000000),
	float2(1.000000, 0.000000),
	float2(0.000000, -1.000000),
	float2(-1.000000, -1.000000),
	float2(-2.000000, 0.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, 0.000000),
	float2(1.000000, 1.000000),
	float2(-2.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-1.000000, -2.000000),
	float2(-3.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, -2.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(-1.000000, 1.000000),
	float2(0.000000, 1.000000),
	float2(-2.000000, 2.000000),
	float2(1.000000, 1.000000),
	float2(-2.000000, -1.000000),
	float2(-1.000000, -4.000000),
	float2(1.000000, -2.000000),
	float2(0.000000, -1.000000),
	float2(-2.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(0.000000, 1.000000),
	float2(2.000000, 1.000000),
	float2(2.000000, 0.000000),
	float2(-1.000000, 2.000000),
	float2(-2.000000, -2.000000),
	float2(0.000000, 2.000000),
	float2(3.000000, 1.000000),
	float2(-3.000000, 2.000000),
	float2(-1.000000, -1.000000),
	float2(1.000000, 0.000000),
	float2(-1.000000, -2.000000),
	float2(1.000000, -1.000000),
	float2(1.000000, 1.000000),
	float2(0.000000, -1.000000),
	float2(-1.000000, 0.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, -1.000000),
	float2(0.000000, -2.000000),
	float2(0.000000, 2.000000),
	float2(0.000000, 3.000000),
	float2(1.000000, 1.000000),
	float2(3.000000, 1.000000),
	float2(0.000000, 1.000000),
	float2(3.000000, -2.000000),
	float2(2.000000, 1.000000),
	float2(1.000000, -3.000000),
	float2(2.000000, -2.000000),
	float2(1.000000, -2.000000),
	float2(-2.000000, 2.000000),
	float2(0.000000, 0.000000),
	float2(1.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(0.000000, -1.000000),
	float2(-1.000000, 2.000000),
	float2(-2.000000, 1.000000),
	float2(-1.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(0.000000, 1.000000),
	float2(1.000000, 1.000000),
	float2(2.000000, 3.000000),
	float2(2.000000, 0.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, -2.000000),
	float2(0.000000, -2.000000),
	float2(-2.000000, -2.000000),
	float2(0.000000, 0.000000),
	float2(-2.000000, 1.000000),
	float2(0.000000, -1.000000),
	float2(-1.000000, 1.000000),
	float2(1.000000, -3.000000),
	float2(0.000000, 2.000000),
	float2(0.000000, 3.000000),
	float2(0.000000, 1.000000),
	float2(-2.000000, 0.000000),
	float2(1.000000, -1.000000),
	float2(-2.000000, -1.000000),
	float2(-2.000000, -2.000000),
	float2(-1.000000, 0.000000),
	float2(-1.000000, -1.000000),
	float2(1.000000, 0.000000),
	float2(1.000000, -2.000000),
}; // SampleArray4x4x16


#endif

void ConvolveRawExperimentalKernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	// Accumulate the center of the kernel.
	if (KernelConfig.bSampleKernelCenter && !KernelConfig.bDescOrder)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}

	const uint TileSize = 4;
	const uint SampleCount = 16;

	uint2 PixelCoord = uint2(KernelConfig.BufferUV * View.BufferSizeAndInvSize.xy) % TileSize;

	LOOP
	for (uint SampleId = 1; SampleId < SampleCount; SampleId++)
	{
		uint MagicIndex = SampleId + SampleCount * (PixelCoord.x + TileSize * PixelCoord.y);
		
		float2 SampleOffset = SampleArray4x4x16[MagicIndex];
		float2 SampleBufferUV = KernelConfig.BufferUV + (SampleOffset * KernelConfig.KernelSpreadFactor) * KernelConfig.BufferSizeAndInvSize.zw;
		
		SampleAndAccumulateMultiplexedSignals(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			SampleBufferUV,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ false);
	}
	
	// Accumulate the center of the kernel.
	if (KernelConfig.bSampleKernelCenter && KernelConfig.bDescOrder)
	{
		SampleAndAccumulateCenterSampleAsItsOwnCluster(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
}

#endif // COMPILE_RAW_EXPERIMENTAL_KERNEL


//------------------------------------------------------- MAIN ENTRY POINTS

/** Accumulate the center of the kernel when KernelConfig.bSampleKernelCenter == false.
 *
 * RefSceneMetadata and SampleSceneMetadata needs to be uncompressed upfront intentionally to share the uncompression with other
 * part of the shader that might have required uncompression anyway.
 */
void AccumulateRefSampleAsKernelCenter(
	FSSDKernelConfig KernelConfig,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators,
	float2 RefBufferUV,
	FSSDSampleSceneInfos RefSceneMetadata,
	FSSDSignalArray RefMultiplexedSamples,
	FSSDSignalFrequencyArray RefMultiplexedFrequencies)
{
	if (!KernelConfig.bSampleKernelCenter)
	{
		AccumulateSampledMultiplexedSignals(
			KernelConfig,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators,
			RefSceneMetadata,
			RefBufferUV,
			RefSceneMetadata,
			RefMultiplexedSamples,
			RefMultiplexedFrequencies,
			/* KernelWeight = */ 1.0,
			/* bForceSample = */ true,
			/* bIsOutsideFrustum = */ false);
		
		if (KernelConfig.SampleSet == 0xDEADDEAD)
		{
		}
#if COMPILE_BOX_KERNEL
		else if (KernelConfig.SampleSet == SAMPLE_SET_3X3 ||
			KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS ||
			KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS)
		{
			BroadcastAccumulateSquare3x3KernelCenter(
				KernelConfig,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators,
				RefSceneMetadata,
				RefBufferUV,
				RefSceneMetadata,
				RefMultiplexedSamples,
				RefMultiplexedFrequencies);
		}
#endif
	}
}

void AccumulateKernel(
	FSSDKernelConfig KernelConfig,
	FSSDTexture2D SignalBuffer0,
	FSSDTexture2D SignalBuffer1,
	FSSDTexture2D SignalBuffer2,
	FSSDTexture2D SignalBuffer3,
	inout FSSDSignalAccumulatorArray UncompressedAccumulators,
	inout FSSDCompressedSignalAccumulatorArray CompressedAccumulators)
{
	if (KernelConfig.SampleSet == 0xDEADDEAD)
	{
	}
#if COMPILE_BOX_KERNEL
	else if (KernelConfig.SampleSet == SAMPLE_SET_1X1)
	{
		if (KernelConfig.bSampleKernelCenter)
		{
			SampleAndAccumulateCenterSampleAsItsOwnCluster(
				KernelConfig,
				SignalBuffer0,
				SignalBuffer1,
				SignalBuffer2,
				SignalBuffer3,
				/* inout */ UncompressedAccumulators,
				/* inout */ CompressedAccumulators);
		}
	}
	else if (
		KernelConfig.SampleSet == SAMPLE_SET_2X2_BILINEAR ||
		KernelConfig.SampleSet == SAMPLE_SET_2X2_STOCASTIC ||
		KernelConfig.SampleSet == SAMPLE_SET_2X2_ADAPTIVE)
	{
		AccumulateBilinear2x2Kernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
	else if (
		KernelConfig.SampleSet == SAMPLE_SET_3X3 ||
		KernelConfig.SampleSet == SAMPLE_SET_3X3_PLUS ||
		KernelConfig.SampleSet == SAMPLE_SET_3X3_CROSS)
	{
		AccumulateSquare3x3Kernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
	else if (
		KernelConfig.SampleSet == SAMPLE_SET_3X3_SOBEK2018 ||
		KernelConfig.SampleSet == SAMPLE_SET_5X5_WAVELET ||
		KernelConfig.SampleSet == SAMPLE_SET_NXN)
	{
		AccumulateSquareKernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
#endif//  COMPILE_BOX_KERNEL
#if COMPILE_STACKOWIAK_KERNEL
	else if (KernelConfig.SampleSet == SAMPLE_SET_STACKOWIAK_4_SETS)
	{
		ConvolveStackowiakKernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
#endif // COMPILE_STACKOWIAK_KERNEL
#if COMPILE_DISK_KERNEL
	else if (KernelConfig.SampleSet == SAMPLE_SET_HEXAWEB)
	{
		ConvolveDiskKernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
#endif // COMPILE_DISK_KERNEL
#if COMPILE_DIRECTIONAL_KERNEL
	else if (KernelConfig.SampleSet == SAMPLE_SET_DIRECTIONAL_RECT || KernelConfig.SampleSet == SAMPLE_SET_DIRECTIONAL_ELLIPSE)
	{
		ConvolveDirectionalRect(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
#endif // COMPILE_DIRECTIONAL_KERNEL
#if COMPILE_RAW_EXPERIMENTAL_KERNEL
	else if (KernelConfig.SampleSet == SAMPLE_SET_RAW_EXPERIMENTAL_KERNEL)
	{
		ConvolveRawExperimentalKernel(
			KernelConfig,
			SignalBuffer0,
			SignalBuffer1,
			SignalBuffer2,
			SignalBuffer3,
			/* inout */ UncompressedAccumulators,
			/* inout */ CompressedAccumulators);
	}
#endif
} // AccumulateKernel()
