// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSDDefinitions.ush"


//------------------------------------------------------- ENUM VALUES

/** What signal should be outputed. */
	#define REJECTION_INPUT_MODE_1UNNORMALIZED 0
	#define REJECTION_INPUT_MODE_2PRETRANSFORMED_MOMMENTS 1

/** How to perform history rejection. */
	#define HISTORY_REJECTION_DISABLED 0
	#define HISTORY_REJECTION_MINMAX_BOUNDARIES 1
	#define HISTORY_REJECTION_VAR_BOUNDARIES 2


//------------------------------------------------------- CONFIGS

#define TILE_PIXEL_SIZE 8


#define CONFIG_SIGNAL_PROCESSING DIM_SIGNAL_PROCESSING
#define CONFIG_SIGNAL_BATCH_SIZE DIM_SIGNAL_BATCH_SIZE



#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
	#define MAX_SIGNAL_BATCH_SIZE CONFIG_SIGNAL_BATCH_SIZE
	#define SIGNAL_ARRAY_SIZE CONFIG_SIGNAL_BATCH_SIZE

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_MONOCHROMATIC_PENUMBRA
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_MONOCHROMATIC_PENUMBRA
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4
	#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE SIGNAL_TEXTURE_TYPE_FLOAT4

	#define CONFIG_INPUT_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE
	#define CONFIG_HISTORY_TEXTURE_COUNT CONFIG_SIGNAL_BATCH_SIZE

	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_PENUMBRA_HISTORY

	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_VAR_BOUNDARIES

	#define CONFIG_NORMALIZED_INPUT 1

	#define CONFIG_REJECTION_SAMPLE_SET SAMPLE_SET_3X3_PLUS

	// Uses nearest to not leak informations on geometric edges by bilateral. This is OK because just blury greyscale details.
	// TODO(Denoiser): a bit hacky, need find better solution.
	// TODO(Denoiser): cause a regression on spot light for some reasons.
	//#define CONFIG_USE_NEAREST_HISTORY 1

	#if DIM_SIGNAL_BATCH_SIZE > 1
		#define CONFIG_CLAMP_UV_PER_SIGNAL 1
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC
	// Denoise diffuse and specular harmonics at the same time.
	#define MAX_SIGNAL_BATCH_SIZE 2
	#define SIGNAL_ARRAY_SIZE 2
	#undef CONFIG_SIGNAL_BATCH_SIZE
	#define CONFIG_SIGNAL_BATCH_SIZE 2

	#define COMPILE_SIGNAL_COLOR 1

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_POLYCHROMATIC_PENUMBRA
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_POLYCHROMATIC_PENUMBRA
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 2

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT  SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_POLYCHROMATIC_PENUMBRA_HISTORY
	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 2

	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_VAR_BOUNDARIES

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
	#define COMPILE_SIGNAL_COLOR 1

	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 2 // First and Second moment in rejection pre convolution.

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_REFLECTIONS_HISTORY

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 2

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_REFLECTIONS_TAA
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_REFLECTIONS_TAA

	#define CONFIG_REJECTION_SAMPLE_SET SAMPLE_SET_3X3_PLUS

	// Use variance based rejection.
	#define CONFIG_HISTORY_REJECTION             HISTORY_REJECTION_VAR_BOUNDARIES
	#define CONFIG_HISTORY_REJECTION_COLOR_SPACE REFLECTIONS_REJECTION_COLOR_SPACE

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 2 // First and Second moment in rejection pre convolution.

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_AO_HISTORY
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_AO_HISTORY

	#define CONFIG_INPUT_TEXTURE_COUNT 1
	#define CONFIG_HISTORY_TEXTURE_COUNT 1

	#define CONFIG_NORMALIZED_INPUT 1

	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_VAR_BOUNDARIES

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_AO_HISTORY
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_AO_HISTORY
	#define CONFIG_PATCH_PREV_SCENE_DEPTH 1
	#define CONFIG_USE_UNLIT_HISTORY_REJECTION 1

	// Use pre transformed rejection buffer that contains pre transformed momment 1 & 2.
	#if 1
		#define CONFIG_USE_REJECTION_BUFFER               1
		#define CONFIG_SIGNAL_HISTORY_REJECTION_LAYOUT    SIGNAL_BUFFER_LAYOUT_AO_REJECTION
		#define CONFIG_REJECTION_TEXTURE_COUNT            1

		#define CONFIG_REJECTION_INPUT_MODE               REJECTION_INPUT_MODE_2PRETRANSFORMED_MOMMENTS
		#define CONFIG_REJECTION_SAMPLE_SET               SAMPLE_SET_3X3_SOBEK2018
	#else
		#define CONFIG_REJECTION_SAMPLE_SET SAMPLE_SET_3X3
	#endif

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR 1

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_RECONSTRUCTION
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_AND_AO_HISTORY

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 2

	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_MINMAX_BOUNDARIES
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 3

	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC && 0
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR_SH 1

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC
	#define CONFIG_SIGNAL_INPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_INDIRECT_HARMONIC
	#define CONFIG_SIGNAL_OUTPUT_TEXTURE_TYPE   SIGNAL_TEXTURE_TYPE_UINT2

	#define CONFIG_INPUT_TEXTURE_COUNT 4
	#define CONFIG_HISTORY_TEXTURE_COUNT 4

	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_MINMAX_BOUNDARIES
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 4

	// Given it's a spherical harmonic that store directionality, only need position based rejection.
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_SPHERICAL_HARMONIC && 1
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR_SH 1

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_LUMEN_DIFFUSE_HISTORY

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 2

	#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_STOCASTIC
	//#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_MINMAX_BOUNDARIES

	// Given it's a spherical harmonic that store directionality, only need position based rejection.
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_SPHERICAL_HARMONIC
	//#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DISABLED
	//#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_DISABLED

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1

	#define COMPILE_SIGNAL_COLOR 1
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_SSGI_HISTORY_R11G11B10

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 2

	// SSGI doesn't have any bilateral distance computed from hitT, so allow to blur spatially by about the size of the kernel.
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 3.0

	// Stocastically sample history to speed up the reprojection.
	#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_STOCASTIC

	// Reject using variance to smooth highlights further on history rejections.
	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_VAR_BOUNDARIES

	#define CONFIG_REJECTION_SAMPLE_SET SAMPLE_SET_3X3_PLUS

	// Use position and normal based rejection ideal for diffuse.
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE

#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
	#define MAX_SIGNAL_BATCH_SIZE 1
	#define SIGNAL_ARRAY_SIZE 1
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN 1

	#define COMPILE_SIGNAL_COLOR_ARRAY 2

	// Input and output layout.
	#define CONFIG_SIGNAL_INPUT_LAYOUT   SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_INPUT
	#define CONFIG_SIGNAL_HISTORY_LAYOUT SIGNAL_BUFFER_LAYOUT_DIFFUSE_PROBE_HIERARCHY_HISTORY

	#define CONFIG_INPUT_TEXTURE_COUNT 2
	#define CONFIG_HISTORY_TEXTURE_COUNT 3

	// SSGI doesn't have any bilateral distance computed from hitT, so allow to blur spatially by about the size of the kernel.
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 3.0

	// Stocastically sample history to speed up the reprojection.
	//#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_BILINEAR
	//#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_STOCASTIC
	#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_ADAPTIVE

	// Reject using variance to smooth highlights further on history rejections.
	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_MINMAX_BOUNDARIES

	// Use position and normal based rejection ideal for diffuse.
	//#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DIFFUSE
	#define CONFIG_BILATERAL_PRESET BILATERAL_PRESET_DISABLED

	#define CONFIG_PATCH_PREV_SCENE_DEPTH 1
	#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_PROBE_HIERARCHY
	//#define CONFIG_HISTORY_BILATERAL_PRESET BILATERAL_PRESET_DISABLED

#else
	#error Unimplemented signal processing.
#endif


//------------------------------------------------------- CONFIG DISABLED DEFAULTS

/** Whether should clamp the UV individually per texture. */
#ifndef CONFIG_CLAMP_UV_PER_SIGNAL
	#define CONFIG_CLAMP_UV_PER_SIGNAL 0
#endif

/** Whether this pass make use of a tile classification. */
#ifndef CONFIG_USE_TILE_CLASSIFICATION
	#define CONFIG_USE_TILE_CLASSIFICATION 0
#endif

/** Whether this pass uses custom rejection buffer. */
#ifndef CONFIG_USE_REJECTION_BUFFER
	#define CONFIG_USE_REJECTION_BUFFER 0
	#define CONFIG_SIGNAL_HISTORY_REJECTION_LAYOUT CONFIG_SIGNAL_INPUT_LAYOUT
#endif

/** Input mode of the rejection signals. */
#ifndef CONFIG_REJECTION_INPUT_MODE
	#define CONFIG_REJECTION_INPUT_MODE REJECTION_INPUT_MODE_1UNNORMALIZED
#endif

/** Sample set to use for history rejection. */
#ifndef CONFIG_REJECTION_SAMPLE_SET
	#define CONFIG_REJECTION_SAMPLE_SET SAMPLE_SET_NXN
#endif

/** The color space to use for history rejection. */
#ifndef CONFIG_REJECTION_BUFFER_COLOR_SPACE
	#define CONFIG_REJECTION_BUFFER_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif

/** The number of signal that should be processed per signal domain. */
#ifndef CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN
	#define CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN SIGNAL_ARRAY_SIZE
#endif

/** Whether can use a nearest sampler when sampling the history */
#ifndef CONFIG_HISTORY_KERNEL
	#define CONFIG_HISTORY_KERNEL SAMPLE_SET_2X2_BILINEAR
#endif

/** Adds a multiplier on how the distance should be computed. */
#ifndef CONFIG_BILATERAL_DISTANCE_MULTIPLIER
	#define CONFIG_BILATERAL_DISTANCE_MULTIPLIER 1.0
#endif

/** Whether should consider the depth velocity to compute the reference scene depth. */
#ifndef CONFIG_PATCH_PREV_SCENE_DEPTH
	#define CONFIG_PATCH_PREV_REF_SCENE_DEPTH 0
#endif

#ifndef CONFIG_NORMALIZED_INPUT
	#define CONFIG_NORMALIZED_INPUT 0
#endif

/** Whether should reject history on pixels with UNLIT shading model. */
#ifndef CONFIG_USE_UNLIT_HISTORY_REJECTION
	#define CONFIG_USE_UNLIT_HISTORY_REJECTION 0
#endif

//------------------------------------------------------- CONFIG ENABLED DEFAULTS

/** Whether should do history rejection. */
#ifndef CONFIG_HISTORY_REJECTION
	#define CONFIG_HISTORY_REJECTION HISTORY_REJECTION_DISABLED
#endif

/** The color space to use for history rejection. */
#ifndef CONFIG_HISTORY_REJECTION_COLOR_SPACE
	#define CONFIG_HISTORY_REJECTION_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif

/** The color space to use for history blending. */
#ifndef CONFIG_HISTORY_BLENDING_COLOR_SPACE
	#define CONFIG_HISTORY_BLENDING_COLOR_SPACE STANDARD_BUFFER_COLOR_SPACE
#endif


//------------------------------------------------------- COMPILATION CONFIGURATION

#define COMPILE_BOX_KERNEL 1

// Moment 1 needed for previous frame sampling.
#define COMPILE_MOMENT1_ACCUMULATOR 1

#if CONFIG_HISTORY_REJECTION != HISTORY_REJECTION_DISABLED
	#define COMPILE_MIN_FREQUENCY_ACCUMULATOR 1
#endif

#if CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_MINMAX_BOUNDARIES
	#define COMPILE_MINMAX_ACCUMULATOR 1
#elif CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_VAR_BOUNDARIES && CONFIG_REJECTION_INPUT_MODE == REJECTION_INPUT_MODE_1UNNORMALIZED
	#define COMPILE_MOMENT2_ACCUMULATOR 1
#endif


//------------------------------------------------------- INCLUDES

#include "SSDSignalFramework.ush"
#include "SSDSignalArray.ush"
#include "SSDSpatialKernel.ush"
#include "../Random.ush"


//------------------------------------------------------- PARAMETERS

DECLARE_SCALAR_ARRAY(int, bCameraCut, MAX_SIGNAL_BATCH_SIZE);

float HistoryPreExposureCorrection;

float4 ScreenPosToHistoryBufferUV;
float4 HistoryBufferSizeAndInvSize;
float4 HistoryBufferUVMinMax;
float4 HistoryBufferScissorUVMinMax[CONFIG_SIGNAL_BATCH_SIZE];


#if !defined(CONFIG_INPUT_TEXTURE_COUNT)
	#error Missing CONFIG_INPUT_TEXTURE_COUNT
#endif

FSSDTexture2D SignalInput_Textures_0;

#if CONFIG_INPUT_TEXTURE_COUNT > 1
FSSDTexture2D SignalInput_Textures_1;
#else
#define SignalInput_Textures_1 SignalInput_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 2
FSSDTexture2D SignalInput_Textures_2;
#else
#define SignalInput_Textures_2 SignalInput_Textures_0
#endif

#if CONFIG_INPUT_TEXTURE_COUNT > 3
FSSDTexture2D SignalInput_Textures_3;
#else
#define SignalInput_Textures_3 SignalInput_Textures_0
#endif


#if CONFIG_USE_REJECTION_BUFFER
	#if !defined(CONFIG_REJECTION_TEXTURE_COUNT)
		#error Missing CONFIG_REJECTION_TEXTURE_COUNT
	#endif

	FSSDTexture2D HistoryRejectionSignal_Textures_0;

	#if CONFIG_REJECTION_TEXTURE_COUNT > 1
	FSSDTexture2D HistoryRejectionSignal_Textures_1;
	#else
	#define HistoryRejectionSignal_Textures_1 HistoryRejectionSignal_Textures_0
	#endif

	#if CONFIG_REJECTION_TEXTURE_COUNT > 2
	FSSDTexture2D HistoryRejectionSignal_Textures_2;
	#else
	#define HistoryRejectionSignal_Textures_2 HistoryRejectionSignal_Textures_0
	#endif

	#if CONFIG_REJECTION_TEXTURE_COUNT > 3
	FSSDTexture2D HistoryRejectionSignal_Textures_3;
	#else
	#define HistoryRejectionSignal_Textures_3 HistoryRejectionSignal_Textures_0
	#endif
#endif // CONFIG_USE_REJECTION_BUFFER


#if !defined(CONFIG_HISTORY_TEXTURE_COUNT)
	#error Missing CONFIG_HISTORY_TEXTURE_COUNT
#endif

FSSDTexture2D PrevHistory_Textures_0;
FSSDRWTexture2D	SignalHistoryOutput_UAVs_0;

#if CONFIG_HISTORY_TEXTURE_COUNT > 1
FSSDTexture2D PrevHistory_Textures_1;
FSSDRWTexture2D	SignalHistoryOutput_UAVs_1;
#else
#define PrevHistory_Textures_1 PrevHistory_Textures_0
#define SignalHistoryOutput_UAVs_1 SignalHistoryOutput_UAVs_0
#endif

#if CONFIG_HISTORY_TEXTURE_COUNT > 2
FSSDTexture2D PrevHistory_Textures_2;
FSSDRWTexture2D	SignalHistoryOutput_UAVs_2;
#else
#define PrevHistory_Textures_2 PrevHistory_Textures_0
#define SignalHistoryOutput_UAVs_2 SignalHistoryOutput_UAVs_0
#endif

#if CONFIG_HISTORY_TEXTURE_COUNT > 3
FSSDTexture2D PrevHistory_Textures_3;
FSSDRWTexture2D	SignalHistoryOutput_UAVs_3;
#else
#define PrevHistory_Textures_3 PrevHistory_Textures_0
#define SignalHistoryOutput_UAVs_3 SignalHistoryOutput_UAVs_0
#endif


//------------------------------------------------------- FUNCTIONS

FSSDSignalSample TransformInputBufferForPreRejection(FSSDSignalSample Sample)
{
	return TransformSignal(
		Sample,
		/* SrcConfig = */ STANDARD_BUFFER_COLOR_SPACE,
		/* DestConfig = */ CONFIG_HISTORY_REJECTION_COLOR_SPACE);
}

FSSDSignalSample TransformSignalForPostRejection(FSSDSignalSample Sample)
{
	return TransformSignal(
		Sample,
		/* SrcConfig = */ CONFIG_HISTORY_REJECTION_COLOR_SPACE,
		/* DestConfig = */ STANDARD_BUFFER_COLOR_SPACE);
}

/** Temporally accumulate the input. */
void TemporallyAccumulate(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	// Find out scene buffer UV.
	float2 SceneBufferUV = DispatchThreadId * ThreadIdToBufferUV.xy + ThreadIdToBufferUV.zw;
	if (true)
	{
		SceneBufferUV = clamp(SceneBufferUV, DenoiserBufferBilinearUVMinMax.xy, DenoiserBufferBilinearUVMinMax.zw);
	}

	// Sample current frame data.
	FSSDCompressedSceneInfos CompressedRefSceneMetadata = SampleCompressedSceneMetadata(
		/* bPrevFrame = */ false,
		SceneBufferUV, BufferUVToBufferPixelCoord(SceneBufferUV));

	float DeviceZ;
	uint ShadingModelID;
	{
		FSSDSampleSceneInfos RefInfo = UncompressSampleSceneInfo(
			CONFIG_METADATA_BUFFER_LAYOUT, /* bIsPrevFrame = */ false,
			DenoiserBufferUVToScreenPosition(SceneBufferUV),
			CompressedRefSceneMetadata);

		DeviceZ = RefInfo.DeviceZ;
		ShadingModelID = RefInfo.ShadingModelID;
	}

	// Reproject to previous frame.
	float3 HistoryScreenPosition = float3(DenoiserBufferUVToScreenPosition(SceneBufferUV), DeviceZ);
	bool bIsDynamicPixel = false;
	if (1)
	{

		float4 ThisClip = float4(HistoryScreenPosition, 1);

		float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
		float3 PrevScreen = PrevClip.xyz * rcp(PrevClip.w);
		float3 Velocity = HistoryScreenPosition - PrevScreen;

		if (1)
		{
			float4 EncodedVelocity = GBufferVelocityTexture.SampleLevel(GlobalPointClampedSampler, SceneBufferUV, 0);
			bIsDynamicPixel = EncodedVelocity.x > 0.0;

			if (bIsDynamicPixel)
			{
				Velocity = DecodeVelocityFromTexture(EncodedVelocity);
			}
		}

		HistoryScreenPosition -= Velocity;
	}


	FSSDSignalArray CurrentFrameSamples;
	FSSDSignalFrequencyArray CurrentFrameFrequencies;
	SampleMultiplexedSignals(
		SignalInput_Textures_0,
		SignalInput_Textures_1,
		SignalInput_Textures_2,
		SignalInput_Textures_3,
		GlobalPointClampedSampler,
		CONFIG_SIGNAL_INPUT_LAYOUT,
		/* MultiplexedSampleId = */ 0,
		/* bNormalizeSample = */ CONFIG_NORMALIZED_INPUT != 0,
		SceneBufferUV,
		/* out */ CurrentFrameSamples,
		/* out */ CurrentFrameFrequencies);

	// Sample the history buffer.
	FSSDSignalArray HistorySamples = CreateSignalArrayFromScalarValue(0.0);
	{
		float2 HistoryBufferUV = HistoryScreenPosition.xy * ScreenPosToHistoryBufferUV.xy + ScreenPosToHistoryBufferUV.zw;
		float2 ClampedHistoryBufferUV = clamp(HistoryBufferUV, HistoryBufferUVMinMax.xy, HistoryBufferUVMinMax.zw);
		bool bIsPreviousFrameOffscreen = any(HistoryBufferUV != ClampedHistoryBufferUV);

		// TODO(Denoiser): global camera cut to save perf that is going to be required for panic post filtering.
		BRANCH
		if (!bIsPreviousFrameOffscreen)
		{
			FSSDKernelConfig KernelConfig = CreateKernelConfig();

			#if DEBUG_OUTPUT
			{
				KernelConfig.DebugPixelPosition = DispatchThreadId;
				KernelConfig.DebugEventCounter = 0;
			}
			#endif
			
			// compile time configuration of the kernel.
			KernelConfig.SampleSet = CONFIG_HISTORY_KERNEL;
			//KernelConfig.SampleSet = SAMPLE_SET_1X1;
			KernelConfig.bSampleKernelCenter = true;
			KernelConfig.BufferLayout = CONFIG_SIGNAL_HISTORY_LAYOUT;
			KernelConfig.MultiplexedSignalsPerSignalDomain = CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
			KernelConfig.bUnroll = true;
			KernelConfig.bPreviousFrameMetadata = true;
			KernelConfig.BilateralDistanceComputation = SIGNAL_WORLD_FREQUENCY_MIN_METADATA;
			KernelConfig.bClampUVPerMultiplexedSignal = CONFIG_CLAMP_UV_PER_SIGNAL != 0;

			// Allow a little bit of error when doing bilateral rejection of the history, to forgive per frame TAA jitter.
			KernelConfig.WorldBluringDistanceMultiplier = max(CONFIG_BILATERAL_DISTANCE_MULTIPLIER, 3.0);
			
			SetBilateralPreset(CONFIG_HISTORY_BILATERAL_PRESET, /* inout */ KernelConfig);

			// SGPR configuration of the kernel.
			KernelConfig.BufferSizeAndInvSize = HistoryBufferSizeAndInvSize;
			KernelConfig.BufferBilinearUVMinMax = HistoryBufferUVMinMax;
			
			#if CONFIG_CLAMP_UV_PER_SIGNAL
			{
				UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
				for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
				{
					uint MultiplexId = BatchedSignalId / CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
					KernelConfig.PerSignalUVMinMax[MultiplexId] = HistoryBufferScissorUVMinMax[MultiplexId];
				}
			}
			#endif
			
			// VGPR configuration of the kernel.
			KernelConfig.BufferUV = HistoryBufferUV + BufferUVBilinearCorrection;
			KernelConfig.bIsDynamicPixel = bIsDynamicPixel;

			#if CONFIG_PATCH_PREV_SCENE_DEPTH
			{
				KernelConfig.RefBufferUV = HistoryBufferUV;
				KernelConfig.RefSceneMetadataLayout = CONFIG_METADATA_BUFFER_LAYOUT;
				KernelConfig.bPreviousFrameRefMetadata = true;

				FSSDSampleSceneInfos PrevRefInfo = UncompressSampleSceneInfo(
					CONFIG_METADATA_BUFFER_LAYOUT, /* bIsPrevFrame = */ false,
					BufferUVToBufferPixelCoord(SceneBufferUV),
					CompressedRefSceneMetadata);

				PrevRefInfo.ScreenPosition = HistoryScreenPosition.xy;
				PrevRefInfo.DeviceZ = HistoryScreenPosition.z;
				PrevRefInfo.WorldDepth = ConvertFromDeviceZ(HistoryScreenPosition.z);
				
				float4 ClipPosition = float4(GetScreenPositionForProjectionType(HistoryScreenPosition.xy, PrevRefInfo.WorldDepth), PrevRefInfo.WorldDepth, 1);

				float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
				PrevRefInfo.TranslatedWorldPosition = mul(ClipPosition, View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;

				KernelConfig.CompressedRefSceneMetadata = CompressSampleSceneInfo(
					KernelConfig.RefSceneMetadataLayout,
					PrevRefInfo);
			}
			#else
			{
				KernelConfig.CompressedRefSceneMetadata = CompressedRefSceneMetadata;
				KernelConfig.RefBufferUV = SceneBufferUV;
				KernelConfig.RefSceneMetadataLayout = CONFIG_METADATA_BUFFER_LAYOUT;
			}
			#endif


			// Compute random signals.
			ISOLATE
			{
				KernelConfig.Randoms[0] = InterleavedGradientNoise(SceneBufferUV * BufferUVToOutputPixelPosition, View.StateFrameIndexMod8);
			}
			
			FSSDSignalAccumulatorArray SignalAccumulators = CreateSignalAccumulatorArray();
			FSSDCompressedSignalAccumulatorArray UnusedCompressedAccumulators = CreateUninitialisedCompressedAccumulatorArray();

			AccumulateKernel(
				KernelConfig,
				PrevHistory_Textures_0,
				PrevHistory_Textures_1,
				PrevHistory_Textures_2,
				PrevHistory_Textures_3,
				/* inout */ SignalAccumulators,
				/* inout */ UnusedCompressedAccumulators);
		
			// Exports the history sample from accumulator.
			{
				UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
				for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
				{
					HistorySamples.Array[BatchedSignalId] = SignalAccumulators.Array[BatchedSignalId].Moment1;
					BRANCH
					if (GET_SCALAR_ARRAY_ELEMENT(bCameraCut, BatchedSignalId))
					{
						HistorySamples.Array[BatchedSignalId] = CreateSignalSampleFromScalarValue(0.0);
					}
				}
			}

			{
				UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
				for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
				{
					FSSDSignalSample CurrentFrameSample = CurrentFrameSamples.Array[BatchedSignalId];
					FSSDSignalSample HistorySample = HistorySamples.Array[BatchedSignalId];

					// Applies correction when pre exposure change happen.
					#if COMPILE_SIGNAL_COLOR
						HistorySamples.Array[BatchedSignalId].SceneColor.rgb *= HistoryPreExposureCorrection;
					#endif

					#if COMPILE_SIGNAL_COLOR_ARRAY > 0
						UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
						for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
						{
							HistorySamples.Array[BatchedSignalId].ColorArray[ColorId] *= HistoryPreExposureCorrection;
						}
					#endif
				} // for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
			}
		} // if (!bIsPreviousFrameOffscreen)
	}
	
	#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY && 0
		DebugOutput[DispatchThreadId] = float4(
			HistorySamples.Array[0].SceneColor.rgb,
			0);
	#endif
	
	const bool bPostRejectionBlending = true;

	// History rejection.
	#if (CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_MINMAX_BOUNDARIES || CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_VAR_BOUNDARIES)
	{
		FSSDKernelConfig KernelConfig = CreateKernelConfig();
		
		#if DEBUG_OUTPUT
		{
			KernelConfig.DebugPixelPosition = DispatchThreadId;
			KernelConfig.DebugEventCounter = 0;
		}
		#endif

		// compile time configuration of the kernel.
		{
			KernelConfig.bSampleKernelCenter = CONFIG_USE_REJECTION_BUFFER != 0;
		
			// History rejection is already fudge factor by how reprojection. Want to prioritize rejection stability with more sample
			// than accuracy, so only take the blur distance of the reference sample that depends on depth and pixel size of the current frame.
			KernelConfig.BilateralDistanceComputation = SIGNAL_WORLD_FREQUENCY_REF_METADATA_ONLY;
			KernelConfig.NeighborToRefComputation = NEIGHBOR_TO_REF_LOWEST_VGPR_PRESSURE;

			// TODO(Denoiser): Should be done for all signals.
			// TODO(Denoiser): Should also take into account the world bluring radius du to size of the pixel.
			if (CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK)
				KernelConfig.BilateralDistanceComputation = SIGNAL_WORLD_FREQUENCY_PRECOMPUTED_BLURING_RADIUS;
			KernelConfig.WorldBluringDistanceMultiplier = CONFIG_BILATERAL_DISTANCE_MULTIPLIER;
			
			#if CONFIG_REJECTION_SAMPLE_SET == REJECTION_SAMPLE_SET_SOBEK2018
			{
				KernelConfig.SampleSet = SAMPLE_SET_3X3_SOBEK2018;
				//KernelConfig.SampleSet = SAMPLE_SET_NXN;
				//KernelConfig.BoxKernelRadius = 3;
				KernelConfig.bUnroll = true;
			}
			#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
			{
				KernelConfig.SampleSet = SAMPLE_SET_3X3_PLUS;
				KernelConfig.bUnroll = true;
			}
			#else
			{
				KernelConfig.SampleSet = (CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK) ? SAMPLE_SET_3X3 : SAMPLE_SET_NXN;
				KernelConfig.BoxKernelRadius = 3;
				KernelConfig.bUnroll = false;
			}
			#endif

			if (CONFIG_USE_REJECTION_BUFFER)
			{
				// History rejection have two moments of the signal being denoised.
				KernelConfig.MultiplexedSignalsPerSignalDomain = 2;
				
				KernelConfig.BufferLayout = CONFIG_SIGNAL_HISTORY_REJECTION_LAYOUT;
				KernelConfig.bNormalizeSample = false;
			
				for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
				{
					KernelConfig.BufferColorSpace[MultiplexId] = CONFIG_REJECTION_BUFFER_COLOR_SPACE;
					KernelConfig.AccumulatorColorSpace[MultiplexId] = CONFIG_HISTORY_REJECTION_COLOR_SPACE;
				}

				// Force to sample the kernel center because will contain two momments for matching scene meta data.
				KernelConfig.bForceKernelCenterAccumulation = true;
			}
			else
			{
				KernelConfig.MultiplexedSignalsPerSignalDomain = CONFIG_MULTIPLEXED_SIGNALS_PER_SIGNAL_DOMAIN;
				KernelConfig.BufferLayout = CONFIG_SIGNAL_INPUT_LAYOUT;
				KernelConfig.bNormalizeSample = true;
			
				for (uint MultiplexId = 0; MultiplexId < SIGNAL_ARRAY_SIZE; MultiplexId++)
				{
					KernelConfig.AccumulatorColorSpace[MultiplexId] = CONFIG_HISTORY_REJECTION_COLOR_SPACE;
				}
			
				if (MAX_SIGNAL_BATCH_SIZE == 1)
				{
					// TODO(Denoiser): missing && abs(PixelOffset.x) <= 1 && abs(PixelOffset.y) <= 1;
					KernelConfig.bForceAllAccumulation = CurrentFrameSamples.Array[0].SampleCount == 0;
				}
			}
			
			SetBilateralPreset(CONFIG_BILATERAL_PRESET, /* inout */ KernelConfig);
		}

		// SGPR configuration of the kernel.
		{
			KernelConfig.BufferSizeAndInvSize = DenoiserBufferSizeAndInvSize;
			KernelConfig.BufferBilinearUVMinMax = DenoiserBufferBilinearUVMinMax;
		}

		// VGPR configuration of the kernel.
		{
			KernelConfig.BufferUV = SceneBufferUV;
			{
				KernelConfig.CompressedRefSceneMetadata = CompressedRefSceneMetadata;
				KernelConfig.RefBufferUV = SceneBufferUV;
				KernelConfig.RefSceneMetadataLayout = CONFIG_METADATA_BUFFER_LAYOUT;
			}
		}

		// Accumulate the current frame to save perf with unecessary bilateral evaluation.
		FSSDSignalAccumulatorArray SignalAccumulators = CreateSignalAccumulatorArray();
		{
			FSSDSampleSceneInfos RefSceneMetadata = UncompressRefSceneMetadata(KernelConfig);
			
			FSSDCompressedSignalAccumulatorArray UnusedCompressedAccumulators = CreateUninitialisedCompressedAccumulatorArray();

			FSSDSignalArray CenterSample = CurrentFrameSamples;
			if (KernelConfig.bNormalizeSample)
			{
				CenterSample = NormalizeToOneSampleArray(CurrentFrameSamples);
			}

			AccumulateRefSampleAsKernelCenter(
				KernelConfig,
				/* inout */ SignalAccumulators,
				/* inout */ UnusedCompressedAccumulators,
				KernelConfig.RefBufferUV,
				RefSceneMetadata,
				CenterSample,
				CurrentFrameFrequencies);
		}

		{
			FSSDCompressedSignalAccumulatorArray UnusedCompressedAccumulators = CreateUninitialisedCompressedAccumulatorArray();

			#if CONFIG_USE_REJECTION_BUFFER
				AccumulateKernel(
					KernelConfig,
					HistoryRejectionSignal_Textures_0,
					HistoryRejectionSignal_Textures_1,
					HistoryRejectionSignal_Textures_2,
					HistoryRejectionSignal_Textures_3,
					/* inout */ SignalAccumulators,
					/* inout */ UnusedCompressedAccumulators);
			#else
				AccumulateKernel(
					KernelConfig,
					SignalInput_Textures_0,
					SignalInput_Textures_1,
					SignalInput_Textures_2,
					SignalInput_Textures_3,
					/* inout */ SignalAccumulators,
					/* inout */ UnusedCompressedAccumulators);
			#endif
		}

		// Clamp history
		UNROLL_N(CONFIG_SIGNAL_BATCH_SIZE)
		for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
		{
			FSSDSignalSample NeighborMoment1 = CreateSignalSampleFromScalarValue(0.0);
			FSSDSignalSample NeighborMoment2 = CreateSignalSampleFromScalarValue(0.0);
			#if CONFIG_REJECTION_INPUT_MODE == REJECTION_INPUT_MODE_1UNNORMALIZED
			{
				float NormalizeFactor = SafeRcp(SignalAccumulators.Array[BatchedSignalId].Moment1.SampleCount);
				NeighborMoment1 = MulSignal(SignalAccumulators.Array[BatchedSignalId].Moment1, NormalizeFactor);
				
				#if COMPILE_MOMENT2_ACCUMULATOR
					NeighborMoment2 = MulSignal(SignalAccumulators.Array[BatchedSignalId].Moment2, NormalizeFactor);
				#endif
			}
			#elif CONFIG_REJECTION_INPUT_MODE == REJECTION_INPUT_MODE_2PRETRANSFORMED_MOMMENTS
			{
				#if SIGNAL_ARRAY_SIZE != 2 * MAX_SIGNAL_BATCH_SIZE
					#error Invalid signal array size.
				#endif

				float NormalizeFactor = SafeRcp(SignalAccumulators.Array[BatchedSignalId * 2 + 0].Moment1.SampleCount);
				NeighborMoment1 = MulSignal(SignalAccumulators.Array[BatchedSignalId * 2 + 0].Moment1, NormalizeFactor);
				NeighborMoment2 = MulSignal(SignalAccumulators.Array[BatchedSignalId * 2 + 1].Moment1, NormalizeFactor);
			}
			#else
				#error NOrmalized samples. 
			#endif
		
			#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS && 0
				FSSDSignalSample Temp = TransformSignalForPostRejection(NeighborMoment1);
				DebugOutput[DispatchThreadId] = float4(
					Temp.SceneColor.rgb,
					0);
			#endif
				
			FSSDSignalSample CurrentFrameSample = CurrentFrameSamples.Array[BatchedSignalId];
			FSSDSignalSample HistorySample = HistorySamples.Array[BatchedSignalId];

			// Clamp the history.
			#if CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_VAR_BOUNDARIES
			{
				#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO
					const float StdDevMultiplier = 6.00;
				#else
					const float StdDevMultiplier = 1.25;
				#endif

				// StdDev = sqrt(abs(NeighborSquareAvg - NeighborAvg * NeighborAvg));
				FSSDSignalSample StdDev = SqrtSignal(AbsSignal(SubtractSignal(NeighborMoment2, PowerSignal(NeighborMoment1, 2))));
				FSSDSignalSample NeighborMin = AddSignal(NeighborMoment1, MulSignal(StdDev, -StdDevMultiplier));
				FSSDSignalSample NeighborMax = AddSignal(NeighborMoment1, MulSignal(StdDev,  StdDevMultiplier));

				if (0)
				{
					FSSDSignalSample QuantizationErrorMin = MulSignal(NeighborMoment1, 1 - SafeRcp(HistorySample.SampleCount));
					FSSDSignalSample QuantizationErrorMax = MulSignal(NeighborMoment1, 1 + SafeRcp(HistorySample.SampleCount));

					NeighborMin = MinSignal(NeighborMin, QuantizationErrorMin);
					NeighborMax = MaxSignal(NeighborMax, QuantizationErrorMax);
				}

				// Transform the history so that it in correct component spaces, and normalized as the clamping box.
				FSSDSignalSample NormalizedHistorySample = NormalizeToOneSample(HistorySample);
				FSSDSignalSample TransformedHistorySample = TransformInputBufferForPreRejection(NormalizedHistorySample);

				// Clamp the history.
				FSSDSignalSample ClampedTransformedHistorySample = ClampSignal(TransformedHistorySample, NeighborMin, NeighborMax);

				// Transform the history back to linear component spaces.
				FSSDSignalSample ClampedHistorySample = TransformSignalForPostRejection(ClampedTransformedHistorySample);

				// Reweight the anti ghosted history.
				{
					FSSDSignalSample RejectedDiff = AbsSignal(SubtractSignal(ClampedTransformedHistorySample, TransformedHistorySample));

					// Compute how much this history get changed.
					float RejectionFactor = 0.0;
					#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS && (CONFIG_HISTORY_REJECTION_COLOR_SPACE & COLOR_SPACE_LCOCG)
					{
						#if !COMPILE_SIGNAL_COLOR
							#error Need to compile signal color.
						#endif
						RejectionFactor = abs(
							Luma_To_LumaLog(ClampedTransformedHistorySample.SceneColor.x) -
							Luma_To_LumaLog(TransformedHistorySample.SceneColor.x));
				
						RejectionFactor = max(RejectionFactor, 1 * max(RejectedDiff.SceneColor.y, RejectedDiff.SceneColor.z));
						RejectionFactor = max(RejectionFactor, 1 * RejectedDiff.SceneColor.a);
					}
					#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_POLYCHROMATIC_PENUMBRA_HARMONIC || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_AO || CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_INDIRECT_AND_AO
					{
						RejectionFactor = abs(ClampedTransformedHistorySample.MissCount - TransformedHistorySample.MissCount);
					}
					#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SSGI
					{
						//float4 ColorDiff = abs(ClampedTransformedHistorySample.SceneColor - TransformedHistorySample.SceneColor) / ClampedTransformedHistorySample.SceneColor;
						//
						//ColorDiff = -min(-ColorDiff, 0.0);
						//
						//RejectionFactor = saturate(max(max(ColorDiff.x, ColorDiff.y), ColorDiff.z));
						RejectionFactor = abs(ClampedTransformedHistorySample.MissCount - TransformedHistorySample.MissCount);
					}
					#else
						#error Unsupported signal rejection.
					#endif
			
					// Compute an initial history weight as if we had removed sample.
					float FinalHistoryWeight = HistorySample.SampleCount * saturate(1 - RejectionFactor);

					// When doing the accumulation before rejection, need to make sure the input weight get through.
					if (!bPostRejectionBlending)
					{
						FinalHistoryWeight = max(FinalHistoryWeight, CurrentFrameSample.SampleCount);
					}

					// When doing upscaling, may have an invalid input sample.
					FinalHistoryWeight = max(FinalHistoryWeight, NeighborMoment1.SampleCount * 0.1);

#if CONFIG_USE_UNLIT_HISTORY_REJECTION
					bool bValid = DenoiseShadingModelID(ShadingModelID);
					if (!bValid)
					{
						FinalHistoryWeight = 0.0;
					}
#endif
					// Apply the history weight.
					HistorySample = MulSignal(ClampedHistorySample, FinalHistoryWeight);
					HistorySample.SampleCount = FinalHistoryWeight;
				}
			}
			#elif CONFIG_HISTORY_REJECTION == HISTORY_REJECTION_MINMAX_BOUNDARIES
			{
				FSSDSignalSample NeighborMin = SignalAccumulators.Array[BatchedSignalId].Min;
				FSSDSignalSample NeighborMax = SignalAccumulators.Array[BatchedSignalId].Max;

				#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_DIFFUSE_PROBE_HIERARCHY
				{
					FSSDSignalSample CenterSample = CurrentFrameSamples.Array[BatchedSignalId];

					FSSDSignalSample CenterMinSample = NormalizeToOneSample(CenterSample);
					FSSDSignalSample CenterMaxSample = NormalizeToOneSample(CenterSample);

					const float Abracadabra = 0.7;

					#if COMPILE_SIGNAL_COLOR
						CenterMinSample.SceneColor *= Abracadabra;
						CenterMaxSample.SceneColor *= rcp(Abracadabra);
					#endif

					#if COMPILE_SIGNAL_COLOR_ARRAY > 0
						UNROLL_N(COMPILE_SIGNAL_COLOR_ARRAY)
						for (uint ColorId = 0; ColorId < COMPILE_SIGNAL_COLOR_ARRAY; ColorId++)
						{
							CenterMinSample.ColorArray[ColorId] *= Abracadabra;
							CenterMaxSample.ColorArray[ColorId] *= rcp(Abracadabra);
						}
					#endif

					NeighborMin = MinSignal(NeighborMin, CenterMinSample);
					NeighborMax = MaxSignal(NeighborMax, CenterMaxSample);
				}
				#endif

				// If no neighborhood had a sample the maximum sample count will remain 0.
				bool bIsValid = NeighborMax.SampleCount > 0.0;

				float RejectedSampleCount = 0;
				HistorySample = MulSignal(TransformSignalForPostRejection(ClampSignal(TransformInputBufferForPreRejection(NormalizeToOneSample(HistorySample)), NeighborMin, NeighborMax)), HistorySample.SampleCount - RejectedSampleCount);

				// All the clamping box was invalid, so invalidate the history sample too.
				FLATTEN
				if (!bIsValid)
				{
					HistorySample = CreateSignalSampleFromScalarValue(0.0);
				}
			}
			#endif

			// Dilate the smallest inv frequency.
			if (1)
			{
				CurrentFrameFrequencies.Array[BatchedSignalId] = MinSignalFrequency(
					CurrentFrameFrequencies.Array[BatchedSignalId],
					SignalAccumulators.Array[BatchedSignalId].MinFrequency);
			}
			
			HistorySamples.Array[BatchedSignalId] = HistorySample;
			CurrentFrameSamples.Array[BatchedSignalId] = CurrentFrameSample;
		} // for (uint BatchedSignalId = 0; BatchedSignalId < CONFIG_SIGNAL_BATCH_SIZE; BatchedSignalId++)
	}
	#endif // CONFIG_HISTORY_REJECTION > 0
	
	{
		UNROLL
		for (uint BatchedSignalId = 0; BatchedSignalId < MAX_SIGNAL_BATCH_SIZE; BatchedSignalId++)
		{
			FSSDSignalSample CurrentFrameSample = CurrentFrameSamples.Array[BatchedSignalId];
			FSSDSignalSample HistorySample = HistorySamples.Array[BatchedSignalId];
			FSSDSignalFrequency CurrentFrequency = CurrentFrameFrequencies.Array[BatchedSignalId];

			float TargetedSampleCount;
			{
				float2 ScreenPosition = DenoiserBufferUVToScreenPosition(SceneBufferUV);
	
				FSSDSampleSceneInfos RefSceneMetadata = UncompressSampleSceneInfo(
					CONFIG_METADATA_BUFFER_LAYOUT, /* bPrevFrame = */ false,
					ScreenPosition, CompressedRefSceneMetadata);

				// Use the diameter, because that is the distance between two pixel.
				float PixelWorldBluringRadius = ComputeWorldBluringRadiusCausedByPixelSize(RefSceneMetadata);
				float WorldBluringRadius = WorldBluringRadiusToBilateralWorldDistance(PixelWorldBluringRadius);
	
				#if CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_SHADOW_VISIBILITY_MASK
				{
					float ResolutionFraction = 0.5;
					float ToleratedNoiseRatio = 0.25 * rcp(9 * sqrt(2));

					float OutputPixelRadius = CurrentFrequency.WorldBluringRadius * rcp(PixelWorldBluringRadius) * ResolutionFraction;

					TargetedSampleCount = clamp(OutputPixelRadius * OutputPixelRadius * (PI * ToleratedNoiseRatio), 1, TARGETED_SAMPLE_COUNT);
				}
				#elif CONFIG_SIGNAL_PROCESSING == SIGNAL_PROCESSING_REFLECTIONS
				{
					float2 NormalizedScreenMajorAxis;
					float InifinityMajorViewportRadius;
					float InifinityMinorViewportRadius;
					ProjectSpecularLobeToScreenSpace(
						RefSceneMetadata,
						/* out */ NormalizedScreenMajorAxis,
						/* out */ InifinityMajorViewportRadius,
						/* out */ InifinityMinorViewportRadius);

					InifinityMajorViewportRadius *= View.ViewSizeAndInvSize.x;
					InifinityMinorViewportRadius *= View.ViewSizeAndInvSize.x;

					TargetedSampleCount = PI * InifinityMajorViewportRadius * InifinityMinorViewportRadius;

					// TODO(Denoiser): try to make this better? * 0.125 to avoid banding caused by floating point precision.
					TargetedSampleCount = clamp(TargetedSampleCount, 1, TARGETED_SAMPLE_COUNT);
				}
				#else
				{
					TargetedSampleCount = TARGETED_SAMPLE_COUNT;
				}
				#endif
			}

			float PreviousFrameWeight = min(HistorySample.SampleCount, TargetedSampleCount - CurrentFrameSample.SampleCount);
			float PreviousFrameMultiplier = HistorySample.SampleCount > 0 ? PreviousFrameWeight / HistorySample.SampleCount : 0;

			// TODO(Denoiser): use WeightedLerp instead for VALU perf.
			// Pre transformation of the signal.
			HistorySample = TransformSignal(
				HistorySample,
				/* SrcBasis = */ STANDARD_BUFFER_COLOR_SPACE,
				/* DestBasis = */ CONFIG_HISTORY_BLENDING_COLOR_SPACE);
			CurrentFrameSample = TransformSignal(
				CurrentFrameSample,
				/* SrcBasis = */ STANDARD_BUFFER_COLOR_SPACE,
				/* DestBasis = */ CONFIG_HISTORY_BLENDING_COLOR_SPACE);

			// Blending.
			HistorySample = AddSignal(MulSignal(HistorySample, PreviousFrameMultiplier), CurrentFrameSample);
		
			// Post transformation of the signal.
			HistorySample = TransformSignal(
				HistorySample,
				/* SrcBasis = */ CONFIG_HISTORY_BLENDING_COLOR_SPACE,
				/* DestBasis = */ STANDARD_BUFFER_COLOR_SPACE);

			//if (View.GeneralPurposeTweak > 0.5)
			HistorySamples.Array[BatchedSignalId] = HistorySample;
		}
	}

	// Romoves NaN and negative numbers
	// HistorySample = Minus(Min(Minus(HistorySample), CreateSignalSampleFromScalarValue(0.0)));
	
	// Masks what should be outputed to ensure the compiler compiles out everything that ends up unneeded.
	uint MultiplexCount = 1;
	FSSDSignalArray OutputSamples = CreateSignalArrayFromScalarValue(0.0);
	FSSDSignalFrequencyArray OutputFrequencies = CreateInvalidSignalFrequencyArray();
	{
		MultiplexCount = CONFIG_SIGNAL_BATCH_SIZE;
		
		UNROLL
		for (uint BatchedSignalId = 0; BatchedSignalId < MultiplexCount; BatchedSignalId++)
		{
			OutputSamples.Array[BatchedSignalId] = HistorySamples.Array[BatchedSignalId];
			OutputFrequencies.Array[BatchedSignalId] = CurrentFrameFrequencies.Array[BatchedSignalId];
		}
	}
	
	// No need to keep DispatchThreadId, while SceneBufferUV is arround at highest VGPR peak because center of the kernel.
	uint2 OutputPixelPostion = BufferUVToBufferPixelCoord(SceneBufferUV);
		
	BRANCH
	if (all(OutputPixelPostion < ViewportMax))
	{
		OutputMultiplexedSignal(
			SignalHistoryOutput_UAVs_0,
			SignalHistoryOutput_UAVs_1,
			SignalHistoryOutput_UAVs_2,
			SignalHistoryOutput_UAVs_3,
			CONFIG_SIGNAL_HISTORY_LAYOUT,
			MultiplexCount,
			OutputPixelPostion,
			OutputSamples,
			OutputFrequencies);
	}
} // TemporallyAccumulate


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_PIXEL_SIZE, TILE_PIXEL_SIZE, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	TemporallyAccumulate(DispatchThreadId, GroupId, GroupThreadId, GroupThreadIndex);
}
