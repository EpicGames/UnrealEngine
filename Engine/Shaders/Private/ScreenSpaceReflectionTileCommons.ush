// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	ScreenSpaceReflectionTileCommons.ush:
=================================================================================================*/
#pragma once

#ifndef SSR_TILE_SIZE_XY
	#define SSR_TILE_SIZE_XY (8U)
#endif

//#if SSR_TILE_SIZE_XY != 8U
//	#error SSR_TILE_SIZE_XY is currently designed to be 8U.
//#endif

bool IsSSRTileUsed(uint2 PixelPos, uint2 InSSRTiledViewRes, StructuredBuffer<uint> InSSRTileMaskBuffer)
{
	uint2 SSRTileIndex = PixelPos / SSR_TILE_SIZE_XY;
	uint MaskLinearIndex = InSSRTiledViewRes.x * SSRTileIndex.y + SSRTileIndex.x;
	uint Mask = 1u << (MaskLinearIndex % 32u);
	bool bTileUsed = (InSSRTileMaskBuffer[MaskLinearIndex / 32u] & Mask) != 0;

	return bTileUsed;
}