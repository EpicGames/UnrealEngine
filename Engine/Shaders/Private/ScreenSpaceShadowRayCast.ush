// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "SceneTexturesCommon.ush"

// Returns distance along ray that the first hit occurred, or negative on miss
float CastScreenSpaceShadowRay(
	float3 RayOriginTranslatedWorld, float3 RayDirection, float RayLength, int NumSteps,
	float Dither, float CompareToleranceScale, bool bHairNoShadowLight,
	out float2 HitUV)
{
	const float4 RayStartClip = mul(float4(RayOriginTranslatedWorld, 1), View.TranslatedWorldToClip);
	const float4 RayDirClip = mul(float4(RayDirection * RayLength, 0), View.TranslatedWorldToClip);
	const float4 RayEndClip = RayStartClip + RayDirClip;

	const float3 RayStart = RayStartClip.xyz / RayStartClip.w;
	const float3 RayEnd = RayEndClip.xyz / RayEndClip.w;

	const float3 RayStep = RayEnd - RayStart;

	const float4 RayDepthClip = RayStartClip + mul(float4(0, 0, RayLength, 0), View.ViewToClip);
	const float3 RayDepth = RayDepthClip.xyz / RayDepthClip.w;

	const float StepOffset = Dither - 0.5f;
	const float Step = 1.0 / NumSteps;

	const float CompareTolerance = abs(RayDepth.z - RayStart.z) * Step * CompareToleranceScale;

	const float StartDepth = LookupDeviceZ(RayStart.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz);

#if SUBSTRATE_ENABLED // TODO: Reconsider this for non-Substrate
	const float FirstSampleTime = StepOffset * Step + Step;

	const float3 SamplePos = RayStart + RayStep * FirstSampleTime;

	float2 SampleUV = SamplePos.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
	float SampleZ = SamplePos.z + CompareTolerance;	// Apply CompareTolerance outside of loop
	float Result = RayLength * FirstSampleTime;

	const float2 SampleUVStep = (RayStep.xy * Step) * View.ScreenPositionScaleBias.xy;
	const float SampleZStep = RayStep.z * Step;	
	const float ResultStep = RayLength * Step;

	UNROLL
	for (int i = 0; i < NumSteps; i++)
	{
		const float SampleDepth = LookupDeviceZ(SampleUV);

		// Avoid self-intersection with the start pixel (exact comparison due to point sampling depth buffer)
		// Exception is made for hair for occluding transmitted light with non-shadow casting light
		if (abs(SampleZ - SampleDepth) < CompareTolerance && (SampleDepth != StartDepth || bHairNoShadowLight))
		{
			HitUV = SampleUV;

			// Off screen masking
			bool bValidPos = all( abs(SampleUV - View.ScreenPositionScaleBias.wz) < abs(View.ScreenPositionScaleBias.xy) );
			return bValidPos ? Result : -1.0;
		}

		SampleUV += SampleUVStep;
		SampleZ += SampleZStep;
		Result += ResultStep;
	}
#else
	float SampleTime = StepOffset * Step + Step;

	UNROLL
	for (int i = 0; i < NumSteps; i++)
	{
		// SamplePos is in NDC space of the current view
		const float3 SamplePos = RayStart + RayStep * SampleTime;
		const float2 SampleUV = SamplePos.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;
		const float SampleDepth = LookupDeviceZ(SampleUV);

		// Avoid self-intersection with the start pixel (exact comparison due to point sampling depth buffer)
		// Exception is made for hair for occluding transmitted light with non-shadow casting light
		if (SampleDepth != StartDepth || bHairNoShadowLight)
		{
			const float DepthDiff = SamplePos.z - SampleDepth;
			const bool bHit = abs(DepthDiff + CompareTolerance) < CompareTolerance;

			if (bHit)
			{
				HitUV = SampleUV;

				// Off screen masking, check NDC position against NDC boundary [-1,1]
				bool bValidPos = all(and(-1.0 < SamplePos.xy, SamplePos.xy < 1.0));

				return bValidPos ? (RayLength * SampleTime) : -1.0;
			}
		}

		SampleTime += Step;
	}
#endif

	return -1;
}