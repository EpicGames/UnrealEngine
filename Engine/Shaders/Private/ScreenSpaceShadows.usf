// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenSpaceShadows.usf
=============================================================================*/

#if BEND_SSS

#include "/Engine/Public/Platform.ush"

#define INTENSITY_MODE_PRIMITIVE_FLAG 0
#define INTENSITY_MODE_DEPTH_BASED_APPROXIMATION 1
#define INTENSITY_MODE_FORCE_CASTING_INTENSITY 2

#define WAVE_SIZE 64
#define SAMPLE_COUNT 60
#define HARD_SHADOW_SAMPLES 4
#define FADE_OUT_SAMPLES 8
#define TRACK_HIT_UV (DIM_INTENSITY_MODE == INTENSITY_MODE_PRIMITIVE_FLAG)

#include "bend_sss_gpu.ush"

float2 InvDepthTextureSize;
float4 LightCoordinate;
int3 WaveOffset;

RWTexture2D<float> OutputTexture;
SamplerState PointBorderSampler;

DispatchParameters SetupBendDispatchParameters(Texture2D DepthTexture, float NearDepthVal, float FarDepthVal, float SurfaceThickness)
{
	DispatchParameters Parameters;
	Parameters.SetDefaults();

	Parameters.LightCoordinate = LightCoordinate;
	Parameters.WaveOffset = WaveOffset.xy;

	Parameters.FarDepthValue = FarDepthVal;
	Parameters.NearDepthValue = NearDepthVal;

	Parameters.InvDepthTextureSize = InvDepthTextureSize;

	Parameters.DepthTexture = DepthTexture;
	Parameters.PointBorderSampler = PointBorderSampler;

	Parameters.SurfaceThickness = SurfaceThickness;

	return Parameters;
}

#endif // BEND_SSS

#include "ScreenSpaceShadowRayCast.ush"

#include "Common.ush"
#include "SceneTexturesCommon.ush"
#include "ComputeShaderUtils.ush"

#include "DeferredShadingCommon.ush"

#ifdef UPSAMPLE_PASS
#	include "ShadowFactorsUpsampleCommon.ush"
#endif

#ifndef THREADGROUP_SIZEX
#	define THREADGROUP_SIZEX 1
#endif

#ifndef THREADGROUP_SIZEY
#	define THREADGROUP_SIZEY 1
#endif

float3 LightDirection;
float ContactShadowLength;
uint bContactShadowLengthInWS;
float ContactShadowCastingIntensity;
float ContactShadowNonCastingIntensity;
float ContactShadowIntensityFadeStart;
float ContactShadowIntensityFadeOneOverLength;
float SurfaceThickness;

uint DownsampleFactor;
int4 ScissorRectMinAndSize;

RWTexture2D<float2> RWShadowFactors;

Texture2D<uint2> StencilTexture;

// Must match C++ (SceneRenderTargets.h)
#define STENCIL_MOBILE_CAST_CONTACT_SHADOW_MASK (1 << 4)

float GetContactShadowOcclusion(float2 HitUV)
{
	const uint2 HitPixelCoord = uint2(HitUV * View.BufferSizeAndInvSize.xy);

#if SHADING_PATH_MOBILE
#	if MOBILE_DEFERRED_SHADING
		FGBufferData SampleGBuffer = MobileFetchAndDecodeGBuffer(HitUV, HitPixelCoord);
		const bool bHitCastContactShadow = CastContactShadow(SampleGBuffer);
#	else
		uint Stencil = StencilTexture.Load(int3(HitPixelCoord, 0)) STENCIL_COMPONENT_SWIZZLE;
		const bool bHitCastContactShadow = Stencil & STENCIL_MOBILE_CAST_CONTACT_SHADOW_MASK;
#	endif
#else
#	if SUBSTRATE_GBUFFER_FORMAT==1
#		if SUBSTRATE_RAYCAST_ENABLED && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
			uint2 PixelPos = View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy * HitUV;
			FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), SubstrateRayCast.MaxBytesPerPixel);
			FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(SubstrateRayCast.MaterialTextureArray, SubstrateAddressing, SubstrateRayCast.TopLayerTexture);
			const bool bHitCastContactShadow = SubstratePixelHeader.DoesCastContactShadow();
#		else
			const bool bHitCastContactShadow = false;
#		endif
#	else
		FGBufferData SampleGBuffer = GetGBufferData(HitUV);
		const bool bHitCastContactShadow = CastContactShadow(SampleGBuffer);
#	endif
#endif

	return bHitCastContactShadow ? ContactShadowCastingIntensity : ContactShadowNonCastingIntensity;
}

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void ScreenSpaceShadowsCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
    uint3 GroupThreadId : SV_GroupThreadID) 
{
	uint ThreadIndex = GroupThreadId.y * THREADGROUP_SIZEX + GroupThreadId.x;

	float2 ScreenUV = (DispatchThreadId.xy * DownsampleFactor + ScissorRectMinAndSize.xy + .5f) * View.BufferSizeAndInvSize.zw;
	float2 ScreenPosition = (ScreenUV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;

	float SceneDepth = CalcSceneDepth(ScreenUV);
	float3 OpaqueTranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;

	const float ContactShadowLengthScreenScale = GetScreenRayLengthMultiplierForProjectionType(SceneDepth).y;
	const float ActualContactShadowLength = ContactShadowLength * (bContactShadowLengthInWS ? 1.0f : ContactShadowLengthScreenScale);

	const float Dither = InterleavedGradientNoise(DispatchThreadId.xy + 0.5f, View.StateFrameIndexMod8);

	float2 HitUV;
	float HitDistance = CastScreenSpaceShadowRay(OpaqueTranslatedWorldPosition, LightDirection, ActualContactShadowLength, 8, Dither, 2.0f, false, HitUV);

	float Result = HitDistance > 0.0 ? (1.0f - GetContactShadowOcclusion(HitUV)) : 1.0f;

	#if METAL_ES3_1_PROFILE 
		// clamp max depth to avoid #inf
		SceneDepth = min(SceneDepth, 65500.0f);
	#endif
	RWShadowFactors[DispatchThreadId.xy] = float2(Result, SceneDepth);
}

#if BEND_SSS

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void ScreenSpaceShadowsBendCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
#if SHADING_PATH_MOBILE
	Texture2D DepthTexture = MobileSceneTextures.SceneDepthTexture;
#else
	Texture2D DepthTexture = SceneTexturesStruct.SceneDepthTexture;
#endif
	DispatchParameters Parameters = SetupBendDispatchParameters(DepthTexture, NearDepthValue, FarDepthValue, SurfaceThickness);

	float Result;
	int2 ResultXY;
	float2 HitUV;
	WriteScreenSpaceShadow(Parameters, GroupId, GroupThreadId.x, Result, ResultXY, HitUV);

#if DIM_INTENSITY_MODE == INTENSITY_MODE_PRIMITIVE_FLAG
	const float ShadowOcclusion = GetContactShadowOcclusion(HitUV);
#elif DIM_INTENSITY_MODE == INTENSITY_MODE_DEPTH_BASED_APPROXIMATION
	const float DeviceZ = DepthTexture[ResultXY].r;
	const float SceneDepth = ConvertFromDeviceZ(DeviceZ);
	const float T = saturate((SceneDepth - ContactShadowIntensityFadeStart) * ContactShadowIntensityFadeOneOverLength);
	const float ShadowOcclusion = lerp(ContactShadowNonCastingIntensity, ContactShadowCastingIntensity, T);
#elif DIM_INTENSITY_MODE == INTENSITY_MODE_FORCE_CASTING_INTENSITY
	const float ShadowOcclusion = ContactShadowCastingIntensity;
#else
#	error "Unsupported DIM_INTENSITY_MODE"
#endif

	OutputTexture[ResultXY] = 1 - (1 - Result) * ShadowOcclusion;
}

#endif // BEND_SSS

#ifdef UPSAMPLE_PASS

Texture2D ShadowFactorsTexture;
SamplerState ShadowFactorsSampler;
float2 ShadowFactorsUVBilinearMax;

float OneOverDownsampleFactor;

void ScreenSpaceShadowsUpsamplePS(
	in float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float Output;
	float SceneDepth;
	UpsampleShadowFactors(SVPos, ScissorRectMinAndSize, OneOverDownsampleFactor, 0, POSITIVE_INFINITY, ShadowFactorsTexture, ShadowFactorsSampler, ShadowFactorsUVBilinearMax, Output, SceneDepth);

	OutColor = EncodeLightAttenuation(half4(Output, Output, Output, Output));
}

#endif // UPSAMPLE_PASS
