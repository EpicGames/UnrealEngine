// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScreenVertexShader.usf: Filter vertex shader source.
=============================================================================*/

#include "Common.ush"

void Main(
	float2 InPosition : ATTRIBUTE0,
	float2 InUV       : ATTRIBUTE1,
	out FScreenVertexOutput Output
	)
{
	DrawRectangle( float4( InPosition, 0, 1 ), InUV, Output.Position, Output.UV);
}

#if USING_LAYERS
struct FScreenVertexLayeredOutput
{
	FScreenVertexOutput Output;
	uint FaceIndex : TEXCOORD1;

	/** Controls which of the cube map faces to rasterize the primitive to, only the value from the first vertex is used. */
	uint RTIndex : SV_RenderTargetArrayIndex;
};
#endif

void MainForGS(
	float2 InPosition : ATTRIBUTE0,
	float2 InUV       : ATTRIBUTE1,
#if USING_LAYERS
	in uint InstanceId	: SV_InstanceID,
	out FScreenVertexLayeredOutput GSOutput
#else
	out FScreenVertexOutput Output
#endif
	)
{
#if USING_LAYERS
	GSOutput.RTIndex = InstanceId;
	GSOutput.FaceIndex = InstanceId;

	// this is just a wrapper of the normal function
	Main(InPosition, InUV, GSOutput.Output);
#else
	// this is just a wrapper of the normal function
	Main(InPosition, InUV, Output);
#endif

}
