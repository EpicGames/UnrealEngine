/**
 * Copyright (C) 2012 Jorge Jimenez (jorge@iryoku.com)
 * Copyright (C) 2012 Diego Gutierrez (diegog@unizar.es)
 * All rights reserved.
 *
 * Redistribution and use in source and binary forms, with or without
 * modification, are permitted provided that the following conditions are met:
 * 
 *    1. Redistributions of source code must retain the above copyright notice,
 *       this list of conditions and the following disclaimer.
 *
 *    2. Redistributions in binary form must reproduce the following disclaimer
 *       in the documentation and/or other materials provided with the 
 *       distribution:
 *
 *       "Uses Separable SSS. Copyright (C) 2012 by Jorge Jimenez and Diego
 *        Gutierrez."
 *
 * THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS ``AS 
 * IS'' AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, 
 * THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR 
 * PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL COPYRIGHT HOLDERS OR CONTRIBUTORS 
 * BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR 
 * CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF 
 * SUBSTITUTE GOODS OR SERVICES; LOSS OF USE, DATA, OR PROFITS; OR BUSINESS 
 * INTERRUPTION) HOWEVER CAUSED AND ON ANY THEORY OF LIABILITY, WHETHER IN 
 * CONTRACT, STRICT LIABILITY, OR TORT (INCLUDING NEGLIGENCE OR OTHERWISE) 
 * ARISING IN ANY WAY OUT OF THE USE OF THIS SOFTWARE, EVEN IF ADVISED OF THE 
 * POSSIBILITY OF SUCH DAMAGE.
 *
 * The views and conclusions contained in the software and documentation are 
 * those of the authors and should not be interpreted as representing official
 * policies, either expressed or implied, of the copyright holders.
 */


/**
 *                  _______      _______      _______      _______
 *                 /       |    /       |    /       |    /       |
 *                |   (----    |   (----    |   (----    |   (----
 *                 \   \        \   \        \   \        \   \
 *              ----)   |    ----)   |    ----)   |    ----)   |
 *             |_______/    |_______/    |_______/    |_______/
 *
 *        S E P A R A B L E   S U B S U R F A C E   S C A T T E R I N G
 *
 *                           http://www.iryoku.com/
 *
 * Hi, thanks for your interest in Separable SSS!
 *
 * It's a simple shader composed of two components:
 *
 * 1) A transmittance function, 'SSSSTransmittance', which allows to calculate
 *    light transmission in thin slabs, useful for ears and nostrils. It should
 *    be applied during the main rendering pass as follows:
 *
 *        float3 t = albedo.rgb * lights[i].color * attenuation * spot;
 *        color.rgb += t * SSSSTransmittance(...)
 *
 *    (See 'Main.fx' for more details).
 *
 * 2) A simple two-pass reflectance post-processing shader, 'SSSSBlur*', which
 *    softens the skin appearance. It should be applied as a regular
 *    post-processing effect like bloom (the usual framebuffer ping-ponging):
 *
 *    a) The first pass (horizontal) must be invoked by taking the final color
 *       framebuffer as input, and storing the results into a temporal
 *       framebuffer.
 *    b) The second pass (vertical) must be invoked by taking the temporal
 *       framebuffer as input, and storing the results into the original final
 *       color framebuffer.
 *
 *    Note that This SSS filter should be applied *before* tonemapping.
 *
 * Before including SeparableSSS.h you'll have to setup the target. The
 * following targets are available:
 *         SMAA_HLSL_3
 *         SMAA_HLSL_4
 *         SMAA_GLSL_3
 *
 * For more information of what's under the hood, you can check the following
 * URLs (but take into account that the shader has evolved a little bit since
 * these publications):
 *
 * 1) Reflectance: http://www.iryoku.com/sssss/
 * 2) Transmittance: http://www.iryoku.com/translucency/
 *
 * If you've got any doubts, just contact us!
 */


//-----------------------------------------------------------------------------
// Configurable Defines

/**
 * SSSS_FOV must be set to the value used to render the scene.
 */
//#ifndef SSSS_FOVY
//#define SSSS_FOVY 20.0
//#endif

/**
 * Light diffusion should occur on the surface of the object, not in a screen 
 * oriented plane. Setting SSSS_FOLLOW_SURFACE to 1 will ensure that diffusion
 * is more accurately calculated, at the expense of more memory accesses.
 */
#ifndef SSSS_FOLLOW_SURFACE
#define SSSS_FOLLOW_SURFACE 0
#endif

//-----------------------------------------------------------------------------
// Porting Functions
#define SSSSTexture2D Texture2D
#define SSSSSampleShadowmap(coord)	float4(0,0,0,0)	// todo: not used yet
#define SSSSSampleSceneColor(coord) GetSceneColor(coord)
#define SSSSSampleSceneColorPoint(coord) GetSceneColor(coord) // todo: could be made point
#define SSSSSampleProfileId(coord) GetSubsurfaceProfileId(coord)
#define SSSSLerp(a, b, t) lerp(a, b, t)
#define SSSSMad(a, b, c) mad(a, b, c)
#define SSSSMul(v, m) mul(v, m)
#define SSSS_FLATTEN FLATTEN
#define SSSS_BRANCH BRANCH
#define SSSS_UNROLL UNROLL
#define SSSS_COMPUTESHADER COMPUTESHADER

//-----------------------------------------------------------------------------
// Separable SSS Transmittance Function

// @param translucency This parameter allows to control the transmittance effect. Its range should be 0..1. Higher values translate to a stronger effect.
// @param sssWidth this parameter should be the same as the 'SSSSBlurPS' one. See below for more details.
// @param worldPosition Position in world space.
// @param worldNormal Normal in world space.
// @param light Light vector: lightWorldPosition - worldPosition.
// @param lightViewProjection Regular world to light space matrix.
// @param lightFarPlane Far plane distance used in the light projection matrix.

float3 SSSSTransmittance(float translucency, float sssWidth, float3 worldPosition, float3 worldNormal, float3 light, float4x4 lightViewProjection, float lightFarPlane)
 {
    /**
     * Calculate the scale of the effect.
     */
    float scale = 8.25 * (1.0 - translucency) / sssWidth;
       
    /**
     * First we shrink the position inwards the surface to avoid artifacts:
     * (Note that this can be done once for all the lights)
     */
    float4 shrinkedPos = float4(worldPosition - 0.005 * worldNormal, 1.0);

    /**
     * Now we calculate the thickness from the light point of view:
     */
    float4 shadowPosition = SSSSMul(shrinkedPos, lightViewProjection);
    float d1 = SSSSSampleShadowmap(shadowPosition.xy / shadowPosition.w).r; // 'd1' has a range of 0..1
    float d2 = shadowPosition.z; // 'd2' has a range of 0..'lightFarPlane'
    d1 *= lightFarPlane; // So we scale 'd1' accordingly:
    float d = scale * abs(d1 - d2);

    /**
     * Armed with the thickness, we can now calculate the color by means of the
     * precalculated transmittance profile.
     * (It can be precomputed into a texture, for maximum performance):
     */
    float dd = -d * d;
    float3 profile = float3(0.233, 0.455, 0.649) * exp(dd / 0.0064) +
                     float3(0.1,   0.336, 0.344) * exp(dd / 0.0484) +
                     float3(0.118, 0.198, 0.0)   * exp(dd / 0.187)  +
                     float3(0.113, 0.007, 0.007) * exp(dd / 0.567)  +
                     float3(0.358, 0.004, 0.0)   * exp(dd / 1.99)   +
                     float3(0.078, 0.0,   0.0)   * exp(dd / 7.41);

    /** 
     * Using the profile, we finally approximate the transmitted lighting from
     * the back of the object:
     */
    return profile * saturate(0.3 + dot(light, -worldNormal));
}

//-----------------------------------------------------------------------------
// Query the kernel for both Substrate and the SSS profile
struct FSeparableFilterParameters
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	float3 D3D;
	float ScatterRadius;
	float WorldUnitScale;
#endif
	uint SubsurfaceProfileInt;
};

#if SUBSTRATE_GBUFFER_FORMAT==1
void FillBurleyParameters(FSubstrateSubsurfaceData SSSData,
	inout FSeparableFilterParameters Parameters)
{
	const FBurleyParameter BurleyParameter = GetBurleyParameters(SSSData);
	const float3 S3D = GetScalingFactor3D(BurleyParameter.SurfaceAlbedo.xyz);

	Parameters.D3D = BurleyParameter.DiffuseMeanFreePath.xyz / S3D.xyz;
	Parameters.ScatterRadius = BurleyParameter.DiffuseMeanFreePath.w;
	Parameters.WorldUnitScale = BurleyParameter.WorldUnitScale;
}
#endif

half4 GetSubsurfaceProfileKernel(uint index, FSeparableFilterParameters Parameters)
{
	half4 Kernel = (half4)0.0f;
#if SUBSTRATE_GBUFFER_FORMAT==1
	if (Parameters.SubsurfaceProfileInt == SSS_PROFILE_ID_PERPIXEL)
	{
		Kernel = GetSubsurfaceProfileKernel(index, Parameters.D3D, 
											Parameters.ScatterRadius, 
											Parameters.WorldUnitScale);
	}
	else
	{
		Kernel = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + index,
											Parameters.SubsurfaceProfileInt);
	}
#else
	Kernel = GetSubsurfaceProfileKernel(SSSS_N_KERNELWEIGHTOFFSET + index, 
										Parameters.SubsurfaceProfileInt);
#endif
	return Kernel;
}

//-----------------------------------------------------------------------------
// Separable SSS Reflectance Pixel Shader

// @param texcoord The usual quad texture coordinates.
// @param dir Direction of the blur: First pass:   float2(1.0, 0.0), Second pass:  float2(0.0, 1.0)
// @param initStencil indicates whether the stencil buffer should be initialized. Should be set to 'true' for the first pass if not previously initialized, to enable optimization of the second pass.
float4 SSSSBlurPS(uint2 BufferPos, float2 BufferUV, float2 dir, bool initStencil, float2 Extent)
{
    // Fetch color of current pixel:
    float4 colorM = SSSSSampleSceneColorPoint(BufferUV);

	// we store the depth in alpha
	float OutDepth = colorM.a;

	colorM.a = GetMaskFromDepthInAlpha(colorM.a);

	// we don't need to process pixels that are not part of the subsurafce scattering (optimization, also prevents divide by later on)
	BRANCH if(!colorM.a)
	{
		// todo: need to check for proper clear
//		discard;
		return 0.0f;
	}

	// 0..1
	float SSSStrength = GetSubsurfaceStrength(BufferUV);

#if !SSSS_COMPUTESHADER
	// Initialize the stencil buffer in case it was not already available:
    if (initStencil) // (Checked in compile time, it's optimized away)
        if (SSSStrength < 1 / 256.0f) discard;
#endif
	float SSSScaleX = SubsurfaceParams.x;

	float scale = SSSScaleX / OutDepth;

    // Calculate the final step to fetch the surrounding pixels:
    float2 finalStep = scale * dir;

	// ideally this comes from a half res buffer as well - there are some minor artifacts
	finalStep *= SSSStrength; // Modulate it using the opacity (0..1 range)
    
	FSeparableFilterParameters SeparableFilterParameters;

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(BufferUV);
	// For SSS_PROFILE_ID_PERPIXEL is managed throught the burley passes
	const uint SubsurfaceProfileInt = SubstrateSubSurfaceHeaderGetIsProfile(SSSData.Header) ? SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header) : SSS_PROFILE_ID_PERPIXEL;
	SeparableFilterParameters.SubsurfaceProfileInt = SubsurfaceProfileInt;
	FillBurleyParameters(SSSData, SeparableFilterParameters);
#else

#if ENABLE_PROFILE_ID_CACHE
	const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileIntWithInvalid(BufferUV, colorM.a);
#else
	const FGBufferData GBufferData = GetGBufferData(BufferUV);
	// 0..255, which SubSurface profile to pick
	// ideally this comes from a half res buffer as well - there are some minor artifacts
	const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(GBufferData);
#endif

	SeparableFilterParameters.SubsurfaceProfileInt = SubsurfaceProfileInt;
#endif

    // Accumulate the center sample:
    float3 colorAccum = 0;
	// >0 to avoid division by 0, not 100% correct to not visible
	const float3 initColorInvDiv = 0.00001f;
	float3 colorInvDiv = initColorInvDiv;

	// center sample
	half3  CentralKernelWeight = GetSubsurfaceProfileKernel(0, SeparableFilterParameters).rgb;

	colorInvDiv += CentralKernelWeight;
	colorAccum = colorM.rgb * CentralKernelWeight;
	float3 BoundaryColorBleed = GetSubsurfaceProfileBoundaryColorBleed(SubsurfaceProfileInt);

	// Accumulate the other samples:
    SSSS_UNROLL
	for (int i = 1; i < SSSS_N_KERNELWEIGHTCOUNT; i++) 
	{
		// Kernel.a = 0..SUBSURFACE_KERNEL_SIZE (radius)
		half4 Kernel = GetSubsurfaceProfileKernel(i, SeparableFilterParameters);

		float4 LocalAccum = 0;

		float2 UVOffset = Kernel.a * finalStep;
		// The kernel is symtrical, we want to use that property.
		// Half the GetSubsurfaceProfileKernel() calls (more expensive if done by texture)
		// Half the weighting computations (saves 3mul per lookup sample)
		SSSS_UNROLL
		for (int Side = -1; Side <= 1; Side += 2)
		{
			// Fetch color and depth for current sample:
			float2 LocalUV = BufferUV + UVOffset * Side;
			float4 color = SSSSSampleSceneColor(LocalUV);
#if ENABLE_PROFILE_ID_CACHE
			uint LocalSubsurfaceProfileInt = ExtractSubsurfaceProfileIntWithInvalid(LocalUV,color.a);
#else
			uint LocalSubsurfaceProfileInt = SSSSSampleProfileId(LocalUV);
#endif
			float3 ColorTint = (LocalSubsurfaceProfileInt == SubsurfaceProfileInt || LocalSubsurfaceProfileInt == SSS_PROFILE_ID_INVALID) ? 1.0f : BoundaryColorBleed;


			float LocalDepth = color.a;
			color.a = GetMaskFromDepthInAlpha(color.a);

#if SSSS_FOLLOW_SURFACE == 1
			// If the difference in depth is huge, we weight the sample less or not at all
			float s = saturate(12000.0f / 400000 * SubsurfaceParams.y *
	//        float s = saturate(300.0f/400000 * SubsurfaceParams.y *
				abs(OutDepth - LocalDepth));

			color.a *= 1 - s;
#endif

			color.a *= (LocalSubsurfaceProfileInt != SSS_PROFILE_ID_INVALID) ? 1 : 0;

			// approximation, ideally we would reconstruct the mask with GetMaskFromDepthInAlpha() and do manual bilinear filter
			// needed?
			color.rgb *= color.a * ColorTint;

			// Accumulate left and right 
			LocalAccum += color;
		}

		// Accumulate to final value (left and right sample with the same weight)
		colorAccum += Kernel.rgb * LocalAccum.rgb;
		colorInvDiv += Kernel.rgb * LocalAccum.a;
	}

	// normalize (some samples are rejected because of depth or the other material is no SSS, compensate for that)
	// done for each color channel to avoid color shift
	float3 OutColor = colorAccum / colorInvDiv; 

	// Subsurface is dark if profile id is invalid all around it, which will contribute dark halo to thin geometries surrounded by SSS.
	// Fallback to the pixel color at the center.
	OutColor = select(colorInvDiv == initColorInvDiv, colorM.rgb, OutColor);

	// alpha stored the SceneDepth (0 if there is no subsurface scattering)
    return float4(OutColor, OutDepth);
}

//-----------------------------------------------------------------------------
