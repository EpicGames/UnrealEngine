// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

/*=============================================================================
	SeparateTranslucency.usf: 
=============================================================================*/

void UpdateNearestSample(float Z, float2 UV, float FullResZ, inout float MinDist, inout float2 NearestUV)
{
    float DepthDelta = abs(Z - FullResZ);

	FLATTEN
    if (DepthDelta < MinDist)
    {
        MinDist = DepthDelta;
        NearestUV = UV;
    }
}

struct NearestDepthNeighborUpsamplingResult
{
	bool bUsePointSampler;
	float2 UV;
};
NearestDepthNeighborUpsamplingResult NearestDepthNeighborUpsampling(
	Texture2D<float> LowResDepthTexture,
	Texture2D<float> FullResDepthTexture,
	float2 Position,
	float2 UV, float2 LowResTexelSize)
{
	NearestDepthNeighborUpsamplingResult UpsampleResult;

//@todo - support for all platforms, just skip the GatherRed optimization where not supported
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5

	// The relative depth comparison breaks down at larger distances and incorrectly causes point sampling on the skybox pixels
	float MaxOperationDepth = 2000000.0f;

	// Note: this upsample is specialized for half res to full res
    float4 LowResDepthBuffer = LowResDepthTexture.GatherRed(GlobalBilinearClampedSampler, UV);
    float4 LowResDepth = min(float4(ConvertFromDeviceZ(LowResDepthBuffer.x), ConvertFromDeviceZ(LowResDepthBuffer.y), ConvertFromDeviceZ(LowResDepthBuffer.z), ConvertFromDeviceZ(LowResDepthBuffer.w)), MaxOperationDepth.xxxx);
	float FullResDepth = min(ConvertFromDeviceZ(FullResDepthTexture[uint2(Position.xy)].x), MaxOperationDepth);

	float RelativeDepthThreshold = .1f;

	// Search for the UV of the low res neighbor whose depth is closest to the full res depth
    float MinDist = 1.e8f;

    float2 UV00 = UV - 0.5f * LowResTexelSize;
    float2 NearestUV = UV00;
    UpdateNearestSample(LowResDepth.w, UV00, FullResDepth, MinDist, NearestUV);

    float2 UV10 = float2(UV00.x + LowResTexelSize.x, UV00.y);
    UpdateNearestSample(LowResDepth.z, UV10, FullResDepth, MinDist, NearestUV);

    float2 UV01 = float2(UV00.x, UV00.y + LowResTexelSize.y);
    UpdateNearestSample(LowResDepth.x, UV01, FullResDepth, MinDist, NearestUV);

    float2 UV11 = float2(UV00.x + LowResTexelSize.x, UV00.y + LowResTexelSize.y);
    UpdateNearestSample(LowResDepth.y, UV11, FullResDepth, MinDist, NearestUV);
	 
	float InvFullResDepth = 1.0f / FullResDepth;

    BRANCH
    if (abs(LowResDepth.w - FullResDepth) * InvFullResDepth < RelativeDepthThreshold 
		&& abs(LowResDepth.z - FullResDepth) * InvFullResDepth < RelativeDepthThreshold 
		&& abs(LowResDepth.x - FullResDepth) * InvFullResDepth < RelativeDepthThreshold
		&& abs(LowResDepth.y - FullResDepth) * InvFullResDepth < RelativeDepthThreshold)
    {
		UpsampleResult.bUsePointSampler = false;
		UpsampleResult.UV = UV;
    }
    else
    {
		UpsampleResult.bUsePointSampler = true;
		UpsampleResult.UV = NearestUV;
    }

#else

	UpsampleResult.bUsePointSampler = false;
	UpsampleResult.UV = UV;

#endif
	
	return UpsampleResult;
}
