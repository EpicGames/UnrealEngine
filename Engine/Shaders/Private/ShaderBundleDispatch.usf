// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "WaveOpUtil.ush"

// Allow platforms to fully override header (to change entrypoint and bindings)
#ifndef OVERRIDE_ENTIRE_SHADERBUNDLE_HEADER
#define OVERRIDE_ENTIRE_SHADERBUNDLE_HEADER 0
#endif

#if OVERRIDE_ENTIRE_SHADERBUNDLE_HEADER
#include "/Platform/Private/ShaderBundle.ush"
#else
#ifdef OVERRIDE_SHADERBUNDLE_USH
#include "/Platform/Private/ShaderBundle.ush"
#include "/Engine/Public/RootConstants.ush"

uint GetRecordCount()   { return GetAllRootConstants().x; }
uint3 GetPlatformData() { return GetAllRootConstants().yzw; }

ByteAddressBuffer RecordArgBuffer;
ByteAddressBuffer RecordDataBuffer;
RWByteAddressBuffer RWExecutionBuffer;

#endif

[numthreads(THREADGROUP_SIZEX, 1, 1)]
void DispatchShaderBundleEntry(uint RecordIndex : SV_DispatchThreadID) 
{
#ifdef OVERRIDE_SHADERBUNDLE_USH
	// Note: Make sure the process record call is not under flow control
	ProcessShaderBundleRecord(
		GetPlatformData(),
		RecordIndex,
		GetRecordCount(),
		RecordArgBuffer,
		RecordDataBuffer,
		RWExecutionBuffer);
#endif
}
#endif
