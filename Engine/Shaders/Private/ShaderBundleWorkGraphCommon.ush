// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

#pragma message("UESHADERMETADATA_VERSION 3EA3DFAD-BDA9-4554-B218-E9F05711625D")

/** Record passed to shader bundle work graph nodes .*/
struct FShaderBundleNodeRecord
{
	uint DispatchGridSize : SV_DispatchGrid;
};
