// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderBundleWorkGraphCommon.ush"

/** RecordArgBuffer contains dispatch sizes at 16 byte stride. */
ByteAddressBuffer RecordArgBuffer;

/** This should match the same structure declard in CPP code. */
struct FEntryNodeRecord
{
	uint DispatchGridSize : SV_DispatchGrid;
	uint RecordCount;
	uint3 PlatformData;
};

/* Entry node which reads required group count and launches the shading nodes. */
[Shader("node")]
[NodeLaunch("broadcasting")]
[NodeIsProgramEntry]
[NodeMaxDispatchGrid(MAX_DISPATCHGRID_SIZEX,1,1)]
[NumThreads(THREADGROUP_SIZEX, 1, 1)]
void WorkGraphMainCS(
	uint DispatchIndex : SV_DispatchThreadID,
	DispatchNodeInputRecord<FEntryNodeRecord> InputRecord,

	// We expect to be linked against a ShaderBundle node array.
	[NodeID("ShaderBundleNode", 0)]
	[MaxRecords(THREADGROUP_SIZEX)]
	[UnboundedSparseNodes]
	NodeOutputArray<FShaderBundleNodeRecord> Output
)
{
	const uint ItemIndex = DispatchIndex;
	const uint RecordCount = InputRecord.Get().RecordCount;
	const uint ArgBaseOffset = InputRecord.Get().PlatformData.x;
	const uint ArgStride = InputRecord.Get().PlatformData.y;

	const bool bValidItem = ItemIndex < RecordCount && Output[ItemIndex].IsValid();

	uint DispatchGridSize = 0;
	if (bValidItem)
	{
		const uint ArgOffset  = ArgBaseOffset + DispatchIndex * ArgStride;

#if UE_BINDLESS_ENABLED
		const uint RecordArgBufferHandle = InputRecord.Get().PlatformData.z;
		ByteAddressBuffer BindlessRecordArgBuffer = GetSRVFromHeap(ByteAddressBuffer, RecordArgBufferHandle);
		DispatchGridSize = BindlessRecordArgBuffer.Load(ArgOffset);
#else
		DispatchGridSize = RecordArgBuffer.Load(ArgOffset);
#endif
	}

	const bool bValidDispatch = DispatchGridSize > 0;

	ThreadNodeOutputRecords<FShaderBundleNodeRecord> OutputRecord = Output[ItemIndex].GetThreadNodeOutputRecords(bValidDispatch ? 1 : 0);
	if (bValidDispatch)
	{
  		OutputRecord.Get().DispatchGridSize = DispatchGridSize;
	}
  	OutputRecord.OutputComplete();
}
