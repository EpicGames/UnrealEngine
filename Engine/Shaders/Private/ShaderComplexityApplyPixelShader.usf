// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "ScreenPass.ush"
#include "MiniFontCommon.ush"
#include "QuadOverdraw.ush"
 
SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D InputTexture;
SamplerState InputSampler;

#if READ_QUAD_OVERDRAW

Texture2D<uint> QuadOverdrawTexture;

uint2 QuadOverdrawOffset()
{
	uint3 QuadBufferSize;
	QuadOverdrawTexture.GetDimensions(0, QuadBufferSize.x, QuadBufferSize.y, QuadBufferSize.z);
	return uint2(QuadBufferSize.x / 2, 0);
}

#endif // READ_QUAD_OVERDRAW

float4 ShaderComplexityColors[MAX_NUM_COMPLEXITY_COLORS];

float ShaderComplexityColorCount;
uint bLegend;
uint DebugViewShaderMode;
uint ColorSamplingMethod;
float ComplexityScale;
uint2 UsedQuadTextureSize;

half3 ColorizeComplexity(float Complexity)
{
	Complexity = clamp(Complexity, 0.0f, .999f);

	// Expand the stored complexity from [0, 1/3] into [0, 6]. The first third of the maximum complexity gets 7 unique colors
	// to blend between, and ShaderComplexityColors[6] represents 1/3rd of the maximum complexity.
	float ExpandedComplexity = min(ShaderComplexityColorCount - 2.f, Complexity * 18.f);

	half4 ColorLow = half4(0,0,0,0);
	half4 ColorHigh = half4(0,0,0,0);
	
	// map the complexity to the passed in colors
	if (ExpandedComplexity < 1)
	{
		ColorLow = ShaderComplexityColors[0];
		ColorHigh = ShaderComplexityColors[1];
	}
	else if (ExpandedComplexity < 2)
	{
		ColorLow = ShaderComplexityColors[1];
		ColorHigh = ShaderComplexityColors[2];
	}
	else if (ExpandedComplexity < 3)
	{
		ColorLow = ShaderComplexityColors[2];
		ColorHigh = ShaderComplexityColors[3];
	}
	else if (ExpandedComplexity < 4)
	{
		ColorLow = ShaderComplexityColors[3];
		ColorHigh = ShaderComplexityColors[4];
	}
	else if (ExpandedComplexity < 5)
	{
		ColorLow = ShaderComplexityColors[4];
		ColorHigh = ShaderComplexityColors[5];
	}
	else if (ExpandedComplexity < 6)
	{
		ColorLow = ShaderComplexityColors[5];
		ColorHigh = ShaderComplexityColors[6];
	}
	else
	{
		// Transform complexity from [1/3,1] to [0,2]. ShaderComplexityColors[8] represents the maximum complexity.
		ExpandedComplexity = (Complexity - .33333333f) * (ShaderComplexityColorCount <= 8 ? 1.5f : 3.0f);
		if (ExpandedComplexity <= 1)
		{
			ColorLow = ShaderComplexityColors[6];
			ColorHigh = ShaderComplexityColors[7];
		}
		else
		{
			ColorLow = ShaderComplexityColors[7];
			ColorHigh = ShaderComplexityColors[8];
		}
	}

	// weight between the nearest colors based on the fraction
	return lerp(ColorLow.rgb, ColorHigh.rgb, frac(ExpandedComplexity));
}

half3 ColorizeComplexityLinear(float Complexity)
{
	float ExpandedComplexity = clamp(Complexity, 0.0f, .999f) * (ShaderComplexityColorCount - 1);
	int ColorIndex = floor(ExpandedComplexity);
	return lerp(ShaderComplexityColors[ColorIndex].rgb, ShaderComplexityColors[ColorIndex + 1].rgb, frac(ExpandedComplexity));
}

half3 ColorizeComplexityStep(float Complexity)
{
	int ColorIndex = round(saturate(Complexity) * (ShaderComplexityColorCount - 1));
	return ShaderComplexityColors[ColorIndex].rgb;
}

half3 ColorizeComplexityFromMode(float C)
{
	uint LocalColorSamplingMethod = CS_RAMP;

#if READ_QUAD_OVERDRAW || FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	LocalColorSamplingMethod = ColorSamplingMethod;
#endif

	[branch]
	if (LocalColorSamplingMethod == CS_RAMP)
	{
		return ColorizeComplexity(C);
	}
	[branch]
	if (LocalColorSamplingMethod == CS_LINEAR)
	{
		return ColorizeComplexityLinear(C);
	}
	[branch]
	if (LocalColorSamplingMethod == CS_STAIR)
	{
		return ColorizeComplexityStep(C);
	}

	return 0;
}

void Main(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	FStereoVSOutput StereoOutput,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	int2 PixelPos = (int2)SvPosition.xy;
	float2 ViewLocalUV = float2(UVAndScreenPos.z * 0.5f + 0.5f, 0.5f - 0.5f * UVAndScreenPos.w);

	int2 InputViewportCenter = (int2)(Input_ViewportMin + Input_ViewportSize / 2);
	int2 OutputViewportCenter = (int2)(Output_ViewportMin + Output_ViewportSize / 2);

	// float3(PS/ShaderBudget, VS/ShaderBudget, overdraw)
	float3 SceneColor = Texture2DSampleLevel(InputTexture, InputSampler, UV, 0).rgb;

#if READ_QUAD_OVERDRAW
	[branch]
	if (DebugViewShaderMode == DVSM_QuadComplexity || DebugViewShaderMode == DVSM_ShaderComplexityContainedQuadOverhead || DebugViewShaderMode == DVSM_ShaderComplexityBleedingQuadOverhead)
	{
		float2 QuadID = UsedQuadTextureSize * Output_ViewportSizeInverse * SvPosition.xy;
		SceneColor.r += ComplexityToFloat(QuadOverdrawTexture.Load(int3(QuadID.xy + QuadOverdrawOffset(), 0)));
	}
#endif // READ_QUAD_OVERDRAW

	SceneColor.r *= ComplexityScale;

	// show overdraw
	float Overdraw = SceneColor.b;

	// PS cost
	OutColor = float4(ColorizeComplexityFromMode(SceneColor.r), 0);

	// Mark missing data as invalid, this would happen when shaders are compiling
	if (SceneColor.r == 0 && DebugViewShaderMode != 0)
	{
		int2 PixelPosition = SvPosition.xy;
		float Error = (PixelPosition.x & 0x08) == (PixelPosition.y & 0x08) ? .22 : 0;
		OutColor.rgb = Error;
	}

	if(!bLegend)
	{
		return;
	}

	// crosshair
	{
		float CrossHairMask = PixelPos.x == OutputViewportCenter.x || PixelPos.y == OutputViewportCenter.y;
		float2 DistAbs = abs(PixelPos - OutputViewportCenter);
		float Dist = max(DistAbs.x, DistAbs.y);
		float DistMask = Dist >= 2 && Dist < 17;

		OutColor.xyz = lerp(OutColor.xyz, float3(1, 1, 1), CrossHairMask * DistMask);
	}
	
	// value under crosshair
	float3 CenterViewportHDRColor = InputTexture.Load(int3(InputViewportCenter, 0)).rgb;

#if READ_QUAD_OVERDRAW
	[branch]
	if (DebugViewShaderMode == DVSM_QuadComplexity || DebugViewShaderMode == DVSM_ShaderComplexityContainedQuadOverhead || DebugViewShaderMode == DVSM_ShaderComplexityBleedingQuadOverhead)
	{
		float2 QuadID = UsedQuadTextureSize * Input_ViewportSizeInverse * InputViewportCenter.xy;
		SceneColor.r += ComplexityToFloat(QuadOverdrawTexture.Load(int3(QuadID.xy + QuadOverdrawOffset(), 0)));
	}
#endif
	CenterViewportHDRColor.r *= ComplexityScale;

	// Legend
	{
		const int OffsetFromLeft = 64;
		const int OffsetFromBottom = 52;
		const int Height = 18;

		const int2 Size = int2(Output_ViewportMax.x - Output_ViewportMin.x - OffsetFromLeft * 2, Height);

		const int2 LeftTop = int2(
			Output_ViewportMin.x + OffsetFromLeft,
			Output_ViewportMax.y - OffsetFromBottom - Size.y);

		const int OuterBorder = 1;

		// (0, 0) .. (1, 1)
		float2 InsetPx = PixelPos - LeftTop;
		float2 InsetUV = InsetPx / Size;
	
		float3 LegendColor = ColorizeComplexityFromMode(InsetUV.x);

		float BorderDistance = ComputeDistanceToRect(PixelPos, LeftTop, Size);
		
		// thin black border around the Legend
		OutColor.xyz = lerp(0, OutColor.xyz, saturate(BorderDistance - (OuterBorder + 17)));
		// Legend
		OutColor.xyz = lerp(LegendColor, OutColor.xyz, saturate(BorderDistance - (OuterBorder + 1)));

		[branch]
		if (DebugViewShaderMode == DVSM_QuadComplexity)
		{
			// OverDraw
			int2 Cursor = LeftTop + int2(saturate(CenterViewportHDRColor.r) * (Size.x - 2 * 8), 5);
			PrintCharacter(PixelPos, OutColor.xyz, .2, Cursor, _O_);
			PrintCharacter(PixelPos, OutColor.xyz, .2, Cursor, _D_);
		}
		else
		{
			// PS
			{
				int2 Cursor = LeftTop + int2(saturate(CenterViewportHDRColor.r) * (Size.x - 2 * 8), 0);

				PrintCharacter(PixelPos, OutColor.xyz, 0, Cursor, _P_);
				PrintCharacter(PixelPos, OutColor.xyz, 0, Cursor, _S_);
			}

			// VS
			{
				int2 Cursor = LeftTop + int2(saturate(CenterViewportHDRColor.g) * (Size.x - 2 * 8), 11);
			
				PrintCharacter(PixelPos, OutColor.xyz, 0, Cursor, _V_);
				PrintCharacter(PixelPos, OutColor.xyz, 0, Cursor, _S_);
			}
		}
	}

	OutColor = RETURN_COLOR(OutColor);
}