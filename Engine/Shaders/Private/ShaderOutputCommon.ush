// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef PIXELSHADEROUTPUT_INTERPOLANTS
	#define PIXELSHADEROUTPUT_INTERPOLANTS 0
#endif

#ifndef PIXELSHADEROUTPUT_BASEPASS
	#define PIXELSHADEROUTPUT_BASEPASS 0
#endif

#ifndef PIXELSHADEROUTPUT_MESHDECALPASS
	#define PIXELSHADEROUTPUT_MESHDECALPASS 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT0
	#define PIXELSHADEROUTPUT_MRT0 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT1
	#define PIXELSHADEROUTPUT_MRT1 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT2
	#define PIXELSHADEROUTPUT_MRT2 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT3
	#define PIXELSHADEROUTPUT_MRT3 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT4
	#define PIXELSHADEROUTPUT_MRT4 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT5
	#define PIXELSHADEROUTPUT_MRT5 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT6
	#define PIXELSHADEROUTPUT_MRT6 0
#endif

#ifndef PIXELSHADEROUTPUT_MRT7
	#define PIXELSHADEROUTPUT_MRT7 0
#endif

#ifndef PIXELSHADEROUTPUT_COVERAGE
	#define PIXELSHADEROUTPUT_COVERAGE 0
#endif

#ifndef PIXELSHADEROUTPUT_A2C
	#define PIXELSHADEROUTPUT_A2C 0
#endif

#ifndef PIXELSHADER_EARLYDEPTHSTENCIL
#define PIXELSHADER_EARLYDEPTHSTENCIL
#endif
