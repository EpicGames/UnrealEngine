// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderPrint.ush:
	Include this to be able to use ShaderPrint with the global uniforms and buffers.
=============================================================================*/

#pragma once

// ES3.1 doesn't support buffers in structs.
// FXC compiler can't unroll the Print loops correctly when the buffers are in structs.
// FXC compiler can't use buffers which are conditionally initialized.
#if ES3_1_PROFILE || (COMPILER_HLSL && !COMPILER_DXC)
#define SHADER_PRINT_USE_GLOBAL_RESOURCE 1
#else
#define SHADER_PRINT_USE_GLOBAL_RESOURCE 0
#endif

#ifndef SHADER_PRINT_EXPLICIT_BINDING
#define SHADER_PRINT_EXPLICIT_BINDING 0
#endif

// Shader print data can be stored with 3 differents methods:
// * ShaderPrintData binding: Shader print data are explicity setup on the CPP side, and the ShaderPrintData buffer are used for storing the data
// * Custom binding         : Shader print data are explicity setup on the CPP side, and the ShaderPrintData buffer are stored into a custom buffer)
// * Diagnostic buffer      : Shader print data & buffer are stored into the Diagnostic buffer, which is always bound and does not require any explicit binding. This is only supported on certain platforms	

// Experimental feature, disabled by default
#define SHADER_PRINT_EXPERIMENTAL_DIAGNOSTIC_BUFFER 0

// On platform supporting diagnostic buffer, shader print data are stored into the diagnostic buffer which is always 
// bound. This allows to access shader print functionality in any shader without explicit shader binding
#define SHADER_PRINT_USE_DIAGNOSTIC_BUFFER (PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER && (PIXELSHADER || COMPUTESHADER || RAYGENSHADER) && !SHADER_PRINT_USE_GLOBAL_RESOURCE && SHADER_PRINT_EXPERIMENTAL_DIAGNOSTIC_BUFFER)
// Offset to the ShaderPrint parameters (see FD3D12DiagnosticBuffer in D3D12Device.h , 6 uint x 64 lanes + 2 uints for BreadCrumbMarker) in Uints
#define SHADER_PRINT_DIAGNOSTIC_BUFFER_PARAMETERS_OFFSET (6u * 64u + 2u)
// Offset to the ShaderPrint entries in Uints (12 uints for storing constant parameters)
#define SHADER_PRINT_DIAGNOSTIC_BUFFER_ENTRY_OFFSET (SHADER_PRINT_DIAGNOSTIC_BUFFER_PARAMETERS_OFFSET + 12u)

// Global buffers for ShaderPrint collection.
#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
	// When a shader pass use explicit ShaderPrint parameter binding, it is possible that the only output of that pass is the shader print entry buffer. 
	// When using Diagnostic buffer as output, the regular shader entry output won't be used, and the pass will be culled. 
	// To prevent this we add a dummy write, so that the pass does get culled
	#if SHADER_PRINT_EXPLICIT_BINDING
	RWBuffer<uint> ShaderPrint_RWEntryBuffer;
	#endif

	#define SHADER_PRINT_RWENTRYBUFFER(Ctx, Idx) UEDiagnosticBuffer[SHADER_PRINT_DIAGNOSTIC_BUFFER_ENTRY_OFFSET + (Idx)]
	#define SHADER_PRINT_RWSTATEBUFFER(Ctx, Idx) UEDiagnosticBuffer[SHADER_PRINT_DIAGNOSTIC_BUFFER_ENTRY_OFFSET + (Idx)]
#else
	Buffer<uint> ShaderPrint_StateBuffer;
	RWBuffer<uint> ShaderPrint_RWEntryBuffer;

	// Use global buffers directly instead of using FShaderPrintContext::Buffers.
	#if SHADER_PRINT_USE_GLOBAL_RESOURCE
	#define SHADER_PRINT_RWENTRYBUFFER(Ctx, Idx) ShaderPrint_RWEntryBuffer[Idx]
	#define SHADER_PRINT_RWSTATEBUFFER(Ctx, Idx) ShaderPrint_StateBuffer[Idx]
	#endif
#endif

#include "ShaderPrintCommon.ush"
#include "MiniFontCommon.ush"
#include "PackUnpack.ush"

#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
void PackShaderPrintConfig(FShaderPrintConfig In)
{
	const uint Offset = SHADER_PRINT_DIAGNOSTIC_BUFFER_PARAMETERS_OFFSET;

	UEDiagnosticBuffer[Offset+0] = PackUInt2ToUInt(In.Resolution.x, In.Resolution.y);
	UEDiagnosticBuffer[Offset+1] = PackFloat2ToUInt(In.CursorCoord);
	UEDiagnosticBuffer[Offset+2] = asuint(In.TranslatedWorldOffset.x); 
	UEDiagnosticBuffer[Offset+3] = asuint(In.TranslatedWorldOffset.y); 
	UEDiagnosticBuffer[Offset+4] = asuint(In.TranslatedWorldOffset.z);
	UEDiagnosticBuffer[Offset+5] = PackFloat2ToUInt(In.FontSize);
	UEDiagnosticBuffer[Offset+6] = PackFloat2ToUInt(In.FontSpacing);
	UEDiagnosticBuffer[Offset+7] = In.MaxCharacterCount;
	UEDiagnosticBuffer[Offset+8] = In.MaxSymbolCount;
	UEDiagnosticBuffer[Offset+9] = In.MaxStateCount;
	UEDiagnosticBuffer[Offset+10]= (In.MaxLineCount << 1) | (In.bIsDrawLocked ? 1 : 0);
	UEDiagnosticBuffer[Offset+11]= In.MaxTriangleCount;
}

FShaderPrintConfig UnpackShaderPrintConfig()
{
	const uint Offset = SHADER_PRINT_DIAGNOSTIC_BUFFER_PARAMETERS_OFFSET;

	FShaderPrintConfig Out;
	Out.Resolution 				= UnpackUInt2FromUInt(UEDiagnosticBuffer[Offset+0]);
	Out.CursorCoord 			= UnpackFloat2FromUInt(UEDiagnosticBuffer[Offset+1]);
	Out.TranslatedWorldOffset.x = asfloat(UEDiagnosticBuffer[Offset+2]);
	Out.TranslatedWorldOffset.y = asfloat(UEDiagnosticBuffer[Offset+3]);
	Out.TranslatedWorldOffset.z = asfloat(UEDiagnosticBuffer[Offset+4]);
	Out.FontSize 				= UnpackFloat2FromUInt(UEDiagnosticBuffer[Offset+5]);
	Out.FontSpacing 			= UnpackFloat2FromUInt(UEDiagnosticBuffer[Offset+6]);
	Out.MaxCharacterCount		= UEDiagnosticBuffer[Offset+7];
	Out.MaxSymbolCount			= UEDiagnosticBuffer[Offset+8];
	Out.MaxStateCount			= UEDiagnosticBuffer[Offset+9];
	Out.MaxLineCount			= UEDiagnosticBuffer[Offset+10] >> 1;
	Out.bIsDrawLocked			= (UEDiagnosticBuffer[Offset+10] & 1) != 0;
	Out.MaxTriangleCount		= UEDiagnosticBuffer[Offset+11];

	// Dummy write to avoid the pass to be culled when using explicit ShaderPrint parameter bindings
	#if SHADER_PRINT_EXPLICIT_BINDING
	if (Out.Resolution.x > 15999)
	{
		ShaderPrint_RWEntryBuffer[0] = 0;
	}
	#endif

	return Out;
}
#endif

FShaderPrintBuffers InitShaderPrintContextBuffers()
{
	FShaderPrintBuffers Buffers;
#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
	// Nothing to do as Buffers.RWEntryBuffer is never used when using diagnostic buffer. 
	// This is ensured by the macro SHADER_PRINT_RWENTRYBUFFER, which defines how writing is handled.
#else
	Buffers.StateBuffer = ShaderPrint_StateBuffer;
	Buffers.RWEntryBuffer = ShaderPrint_RWEntryBuffer;
#endif
	return Buffers;
}

// Initialise FShaderPrintConfig from the ShaderPrintData uniform buffer.
FShaderPrintConfig InitShaderPrintContextConfig()
{
#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
	return UnpackShaderPrintConfig();
#else
	FShaderPrintConfig Config;
	Config.Resolution = ShaderPrintData.Resolution;
	Config.CursorCoord = ShaderPrintData.CursorCoord;
	Config.TranslatedWorldOffset = ShaderPrintData.TranslatedWorldOffset;
	Config.FontSize = ShaderPrintData.FontSize;
	Config.FontSpacing = ShaderPrintData.FontSpacing;
	Config.MaxCharacterCount = ShaderPrintData.MaxCharacterCount;
	Config.MaxSymbolCount = ShaderPrintData.MaxSymbolCount;
	Config.MaxStateCount = ShaderPrintData.MaxStateCount;
	Config.MaxLineCount = ShaderPrintData.MaxLineCount;
	Config.MaxTriangleCount = ShaderPrintData.MaxTriangleCount;
	Config.bIsDrawLocked = ShaderPrintData.IsDrawLocked;
	return Config;
#endif
}

// Initialise FShaderPrintContext.
FShaderPrintContext InitShaderPrintContext(bool bActive, float2 InStartPos)
{
	FShaderPrintContext Out;
	Out.bIsActive = bActive && SHADER_PRINT_ALLOW != 0;
	Out.StartPos = InStartPos;
	Out.Pos = InStartPos;
	Out.RectMin = InStartPos;
	Out.RectMax = InStartPos;
	Out.Config = InitShaderPrintContextConfig();
	#if !SHADER_PRINT_USE_GLOBAL_RESOURCE
	Out.Buffers = InitShaderPrintContextBuffers();
	#endif
	return Out;
}

FShaderPrintContext InitShaderPrintContext(bool bActive, uint2 InStartCoord) 
{
	FShaderPrintConfig Config = InitShaderPrintContextConfig();
	return InitShaderPrintContext(bActive, float2(InStartCoord) / float2(Config.Resolution));
}

FShaderPrintContext InitShaderPrintContextAtCursor(uint2 InCurrentCoord, uint2 InStartCoord) 
{
	FShaderPrintConfig Config = InitShaderPrintContextConfig();
	const bool bActive = all(int2(InCurrentCoord) == Config.CursorCoord);
	return InitShaderPrintContext(bActive, float2(InStartCoord) / float2(Config.Resolution));
}

FShaderPrintContext InitShaderPrintContextAtCursor(uint2 InStartCoord) 
{
	return InitShaderPrintContextAtCursor(GetShaderPrintCurrentPixelCoord(), InStartCoord);
}

FShaderPrintContext InitShaderPrintContext()
{
	return InitShaderPrintContext(true, float2(0, 0));
}

// Backwards compatibility for client code that didn't use FShaderPrintContext.
int2 GetCursorPos()
{
	return GetCursorPos(InitShaderPrintContext());
}
void AddFilledTriangleTWS(float3 Pos0, float3 Pos1, float3 Pos2, float4 Color)
{
	AddFilledTriangleTWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Color); 
}
void AddFilledTriangleWS(float3 Pos0, float3 Pos1, float3 Pos2, float4 Color)
{
	AddFilledTriangleWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Color); 
}
void AddFilledQuadTWS(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) 
{ 
	AddFilledQuadTWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Pos3, Color);
}
void AddFilledQuadWS(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) 
{ 
	AddFilledQuadWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Pos3, Color);
}
void AddFilledQuadSS(float2 Pos0, float2 Pos1, float4 Color)
{
	AddFilledQuadSS(InitShaderPrintContext(), Pos0, Pos1, Color);
}
void AddLineTWS(float3 Pos0, float3 Pos1, float4 Color)
{
	AddLineTWS(InitShaderPrintContext(), Pos0, Pos1, Color);
}
void AddLineTWS(float3 Pos0, float3 Pos1, float4 Color0, float4 Color1)
{
	AddLineTWS(InitShaderPrintContext(), Pos0, Pos1, Color0, Color1);
}
void AddLineWS(float3 Pos0, float3 Pos1, float4 Color)
{
	AddLineWS(InitShaderPrintContext(), Pos0, Pos1, Color);
}
void AddLineWS(float3 Pos0, float3 Pos1, float4 Color0, float4 Color1) 
{ 
	AddLineWS(InitShaderPrintContext(), Pos0, Pos1, Color0, Color1);
}
void AddLine(float3 Pos0, float3 Pos1, float4 Color) 
{
	AddLine(InitShaderPrintContext(), Pos0, Pos1, Color);
}
void AddLine(float3 Pos0, float3 Pos1, float4 Color0, float4 Color1) 
{
	AddLine(InitShaderPrintContext(), Pos0, Pos1, Color0, Color1);
}
void AddLineSS(float2 Pos0, float2 Pos1, float4 Color) 
{
	AddLineSS(InitShaderPrintContext(), Pos0, Pos1, Color);
}
void AddQuadWS(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) 
{
	AddQuadWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Pos3, Color);
}
void AddQuadTWS(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) 
{
	AddQuadTWS(InitShaderPrintContext(), Pos0, Pos1, Pos2, Pos3, Color);
}
void AddQuad(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) 
{
	AddQuad(InitShaderPrintContext(), Pos0, Pos1, Pos2, Pos3, Color);
}
void AddQuadSS(float2 MinPos, float2 MaxPos, float4 Color)
{
	AddQuadSS(InitShaderPrintContext(), MinPos, MaxPos, Color);
}
void AddAABBTWS(float3 Min, float3 Max, float4 Color) 
{
	AddAABBTWS(InitShaderPrintContext(), Min, Max, Color);
}
void AddAABBWS(float3 Min, float3 Max, float4 Color)
{
	AddAABBWS(InitShaderPrintContext(), Min, Max, Color);
}
void AddAABB(float3 Min, float3 Max, float4 Color) 
{
	AddAABB(InitShaderPrintContext(), Min, Max, Color);
}
void AddCrossWS(float3 Pos, float Size, float4 Color) 
{
	AddCrossWS(InitShaderPrintContext(), Pos, Size, Color);
}
void AddCrossTWS(float3 Pos, float Size, float4 Color) 
{
	AddCrossTWS(InitShaderPrintContext(), Pos, Size, Color);
}
void AddCross(float3 Pos, float Size, float4 Color) 
{
	AddCross(InitShaderPrintContext(), Pos, Size, Color);
}
void AddReferentialWS(float3 Pos, float3x3 InM, float Scale = 1)
{
	AddReferentialWS(InitShaderPrintContext(), Pos, InM, Scale);
}
void AddReferentialWS(float4x4 InM, float Scale = 1) 
{
	AddReferentialWS(InitShaderPrintContext(), InM, Scale);
}
void AddReferentialWS(float3 Pos, float3 TangentZ, float Scale = 1) 
{
	AddReferentialWS(InitShaderPrintContext(), Pos, TangentZ, Scale);
}
void AddReferentialWS(float3 Pos, float3 T, float3 B, float3 N, float Scale = 1)
{
	AddReferentialWS(InitShaderPrintContext(), Pos, T, B, N, Scale);
}
void AddReferentialTWS(float3 Pos, float3x3 InM, float Scale = 1)
{
	AddReferentialTWS(InitShaderPrintContext(), Pos, InM, Scale);
}
void AddReferentialTWS(float4x4 InM, float Scale = 1) 
{
	AddReferentialTWS(InitShaderPrintContext(), InM, Scale);
}
void AddReferentialTWS(float3 Pos, float3 TangentZ, float Scale = 1) 
{
	AddReferentialTWS(InitShaderPrintContext(), Pos, TangentZ, Scale);
}
void AddReferentialTWS(float3 Pos, float3 T, float3 B, float3 N, float Scale = 1)
{
	AddReferentialTWS(InitShaderPrintContext(), Pos, T, B, N, Scale);
}
void AddReferential(float3 Pos, float3x3 InM, float Scale = 1) 
{
	AddReferential(InitShaderPrintContext(), Pos, InM, Scale);
}
void AddReferential(float4x4 InM, float Scale = 1) 
{
	AddReferential(InitShaderPrintContext(), InM, Scale);
}
void AddReferential(float3 Pos, float3 TangentZ, float Scale = 1) 
{
	AddReferential(InitShaderPrintContext(), Pos, TangentZ, Scale);
}
void AddReferential(float3 Pos, float3 T, float3 B, float3 N, float Scale = 1) 
{
	AddReferential(InitShaderPrintContext(), Pos, T, B, N, Scale);
}
void AddLineTriangleWS(float3 P0, float3 P1, float3 P2, float4 Color) 
{
	AddLineTriangleWS(InitShaderPrintContext(), P0, P1, P2, Color);
}
void AddLineTriangleTWS(float3 P0, float3 P1, float3 P2, float4 Color) 
{
	AddLineTriangleTWS(InitShaderPrintContext(), P0, P1, P2, Color);
}
void AddLineTriangle(float3 P0, float3 P1, float3 P2, float4 Color) 
{
	AddLineTriangle(InitShaderPrintContext(), P0, P1, P2, Color);
}
void AddOBBWS(float3 Min, float3 Max, float4 Color, float4x4 Transform)
{
	AddOBBWS(InitShaderPrintContext(), Min, Max, Color, Transform);
}
void AddOBBTWS(float3 Min, float3 Max, float4 Color, float4x4 Transform)
{
	AddOBBTWS(InitShaderPrintContext(), Min, Max, Color, Transform);
}
void AddOBB(float3 Min, float3 Max, float4 Color, float4x4 Transform) 
{
	AddOBB(InitShaderPrintContext(), Min, Max, Color, Transform);
}

// Verify/Assert (experimental)
// Can be used in any shader binding ShaderPrint uniform buffer. It will print the 
// assert for all pixels/threads not complying to the condition. All print will be 
// stacked correctly.
// 
// Example:
//  verify(DispatchThreadId < MaxThread);
//  verifyf(DispatchThreadId < MaxThread, TEXT("Error: ThreadId higher than MaxThread");
//
// This function also returns a FShaderPrintContext so that one can add more details if needed.
//
// Example:
//  FShaderPrintContext Ctx = verify(DispatchThreadId < MaxThread, TEXT("Error: thread higher MaxThread");
//  Print(Ctx, DispatchThreadId, FontWhite);
//
FShaderPrintContext Assert(bool InCondition, FShaderPrintText InText, uint2 InWritePos = uint2(50, 150))
{
	FShaderPrintContext OutCtx = InitShaderPrintContext(false, InWritePos);
	if (!InCondition)
	{
		uint LineOffset = 0;
		SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(OutCtx, 3) /* Free counter */, 1, LineOffset);
		InWritePos.y += LineOffset * 12;

		OutCtx = InitShaderPrintContext(true, InWritePos);
		Print(OutCtx, InText, FontRed);
	}

	return OutCtx;
}

FShaderPrintContext Assert(bool InCondition, FShaderPrintText InText, FShaderPrintText InFile, uint InLine, FShaderPrintText InSeparator, uint2 InWritePos = uint2(50, 150))
{
	FShaderPrintContext OutCtx = InitShaderPrintContext(false, InWritePos);
	if (!InCondition)
	{
		uint LineOffset = 0;
		SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(OutCtx, 3) /* Free counter */, 1, LineOffset);
		InWritePos.y += LineOffset * 12;

		OutCtx = InitShaderPrintContext(true, InWritePos);
		Print(OutCtx, InText, FontRed);
		Print(OutCtx, InSeparator, FontRed);
		Print(OutCtx, InFile, FontRed);
		Print(OutCtx, InSeparator, FontRed);
		Print(OutCtx, InLine, FontRed);
	}

	return OutCtx;
}

// Non-blocking shader check (unlike check()/checkf() which terminate the GPU execution)
#define verify(Condition) 			Assert(Condition, TEXT(#Condition), TEXT(__FILE__), __LINE__, TEXT(" - "));
#define verifyf(Condition, Message) Assert(Condition, Message, TEXT(__FILE__), __LINE__, TEXT(" - "));