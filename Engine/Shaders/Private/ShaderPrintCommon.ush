// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderPrintCommon.ush:
	Include this to be able to call ShaderPrint() from arbitrary shaders.
=============================================================================*/

#pragma once

#ifndef SHADER_PRINT_ALLOW
	#define SHADER_PRINT_ALLOW 1
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Description
// Shader Print allows to print value & text from shaders. 
// 
// 0. Contex-based state API
// -------------------------
// FShaderPrintContext Ctx = InitShaderPrintContext(all(SvPosition.xy == float2(100, 100)), uint2(50,50));
//                                                      ---------------------------------   ------------
//                                                           Pixel/thread to filter        Coord at which 
//                                                                                            to draw
// 
// 1. How to print value?
// ----------------------
// First create a context, then print scalar/vector/matrix/text...
// 
//   FShaderPrintContext Ctx = InitShaderPrintContext(all(PixelPos.xy == uint2(100, 100)), uint2(50,50));
// 
//   Print(Ctx, MyFloat);
//   Newline(Ctx);
// 
//   Print(Ctx, TEXT("My string for labbelling a float :"), FontOrange);
//   Print(Ctx, MyFloatInYellow, FontYellow);
//   Newline(Ctx);
// 
// Numerical values can be further formatted, by configuring the number of digits spanned by a number 
// (i.e. the number of written&non-written digits), and the number of wanted decimal for floating value.
// 
//   Print(Ctx, 123, FontYellow, 5); // Spanning is show as: .....
//   Print(Ctx, 456, FontYellow, 4); // Spanning is show as: xxxx
//   Print(Ctx, 789, FontYellow, 3); // Spanning is show as: ---
//
// Will be displayed as follow:
//   123   456  789
//   ..... xxxx ---
// 
// Print(Ctx, 12.3456, FontYellow, 5, 2); // Spanning is show as: ...
// Print(Ctx, 12.3456, FontYellow, 5, 1); // Spanning is show as: xxx
// 
// Will be displayed as follow:
//   12.34 12.3
//   ..... xxxxx
// 
// 2. How to add widget?
// ---------------------
// Widget can be added from shader. 
// For slider:
// 
//   const float MySliderValue  = AddSlider(Ctx, TEXT("My Slider"), DefaultValue, GetDefaultFontColor());
// 
// For checkbox:
// 
//   const bool MyCheckboxValue = AddCheckbox(Ctx, TEXT("My Checkbox"), DefaultValue, GetDefaultFontColor());
//
// The value of these widgets can be retrieved/evaluated from *any* thread/pixels, but the 
// drawing need to be done from a single thread/pixel. This is done through the ShaderPrintContext, 
// which setup which pixel/thread is active/filtered. Example with a compute shader:
// 
//    [numthreads(8, 8, 1)]
//    void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
//    {
//       // Create context which 'add' the widget only on thread (0,0)
//       FShaderPrintContext Ctx = InitShaderPrintContext(all(DispatchThreadId == uint2(0, 0)), uint2(50,50));
// 
//		 // Any threads can get the slider value with this call
//       const float ScaleValue = AddSlider(Ctx, TEXT("Scale"), 1.f, GetDefaultFontColor());
//       ...
//    }
// 

///////////////////////////////////////////////////////////////////////////////////////////////////
// Primitive & Symbols layout in RWEntryBuffer
//  ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
// ||  Line    | Triangle |  Symbol |  Empty  ||  Symb0  |  Symb1  |  Symb2  |  Symb3  |  ...  |  Line0  |  Line1  |  Line2  |  Line3  |  ...  | Triangle0 | Triangle1 | Triangle2 | Triangle3 | ...
// || Counter  | Counter  | Counter | Counter ||         |         |         |         |  ...  |         |         |         |         |  ...  |           |           |           |           | ...
// || (1uint)  | (1uint)  | (1uint) | (1uint) || (4uint) | (4uint) | (4uint) | (4uint) |  ...  | (8uint) | (8uint) | (8uint) | (8uint) |  ...  | (12uint)  | (12uint)  | (12uint)  | (12uint)  | ...
//  ----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------

///////////////////////////////////////////////////////////////////////////////////////////////////
// Input types

// Content of FShaderPrintItem.Type defines what to cast FShaderPrintItem.Value as

// Type of primitives
#define SHADER_PRINT_TYPE_SYMBOL   0
#define SHADER_PRINT_TYPE_FLOAT    1
#define SHADER_PRINT_TYPE_INT      2
#define SHADER_PRINT_TYPE_UINT     3
#define SHADER_PRINT_TYPE_HEX      4
#define SHADER_PRINT_TYPE_SLIDER   5
#define SHADER_PRINT_TYPE_CHECK    6
// Type of primitives (to be merged with the list above once all buffer are merged)
#define SHADER_PRINT_TYPE_LINE     0
#define SHADER_PRINT_TYPE_TRIANGLE 1

// Size of packed data, in uint
#define SHADER_PRINT_UINT_COUNT_SYMBOL   4
#define SHADER_PRINT_UINT_COUNT_LINE     8
#define SHADER_PRINT_UINT_COUNT_TRIANGLE 12

// Counter offset
#define SHADER_PRINT_COUNTER_OFFSET_LINE     0
#define SHADER_PRINT_COUNTER_OFFSET_TRIANGLE 1
#define SHADER_PRINT_COUNTER_OFFSET_SYMBOL   2
#define SHADER_PRINT_COUNTER_OFFSET_FREE     3
#define SHADER_PRINT_COUNTER_COUNT           4

uint GetSymbolOffset(uint InIndex)
{
	return SHADER_PRINT_COUNTER_COUNT + SHADER_PRINT_UINT_COUNT_SYMBOL * InIndex; 
}
uint GetPrimitiveLineOffset(uint InIndex, uint MaxCharacterCount)
{
	return SHADER_PRINT_COUNTER_COUNT + SHADER_PRINT_UINT_COUNT_SYMBOL * MaxCharacterCount + SHADER_PRINT_UINT_COUNT_LINE * InIndex; 
}
uint GetPrimitiveTriangleOffset(uint InIndex, uint MaxCharacterCount, uint MaxLineCount)
{
	return SHADER_PRINT_COUNTER_COUNT + SHADER_PRINT_UINT_COUNT_SYMBOL * MaxCharacterCount + SHADER_PRINT_UINT_COUNT_LINE * MaxLineCount + SHADER_PRINT_UINT_COUNT_TRIANGLE * InIndex; 
}

void ClearCounters(RWStructuredBuffer<uint> InRWBuffer)
{
	// Symbol/print Counter
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_LINE] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_TRIANGLE] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_FREE] = 0;
}

void ClearCounters(RWBuffer<uint> InRWBuffer)
{
	// Symbol/print Counter
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_LINE] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_TRIANGLE] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL] = 0;
	InRWBuffer[SHADER_PRINT_COUNTER_OFFSET_FREE] = 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// State 
#define SHADER_PRINT_STATE_COUNT_OFFSET 0
#define SHADER_PRINT_STATE_VALUE_OFFSET 1

#define SHADER_PRINT_STATE_STRIDE 3
#define SHADER_PRINT_STATE_INDEX_METADATA 0
#define SHADER_PRINT_STATE_INDEX_VALUE 1
#define SHADER_PRINT_STATE_INDEX_FRAME 2

#define SHADER_PRINT_STATE_INVALID_INDEX 0xFF
#define SHADER_PRINT_STATE_HASH_MASK 0xFFFFFF

///////////////////////////////////////////////////////////////////////////////////////////////////
// Buffers/Resources accessors

#if !defined(SHADER_PRINT_RWENTRYBUFFER)
#define SHADER_PRINT_RWENTRYBUFFER(Ctx, Idx) Ctx.Buffers.RWEntryBuffer[Idx]
#define SHADER_PRINT_RWSTATEBUFFER(Ctx, Idx) Ctx.Buffers.StateBuffer[Idx]
#endif

// Currently a bug Metal shader translation cause compilation to fails due to atomic operation.
// As a workaround, disabling interlock add on Metal
#if COMPILER_METAL && FEATURE_LEVEL < FEATURE_LEVEL_SM6
#define SHADER_PRINT_INTERLOCKEDADD(InBuffer, InIncrement, OutOldValue)
#else
#define SHADER_PRINT_INTERLOCKEDADD(InBuffer, InIncrement, OutOldValue) InterlockedAdd(InBuffer, InIncrement, OutOldValue)
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Font Color

struct FFontColor
{
	float3 Color;
};

FFontColor InitFontColor(float InX, float InY, float InZ)	{ FFontColor Out; Out.Color = float3(InX, InY, InZ); return Out; }
FFontColor InitFontColor(float3 In)							{ FFontColor Out; Out.Color = In; return Out; }
FFontColor GetDefaultFontColor()							{ FFontColor Out; Out.Color = float3(1,1,1); return Out; }

FFontColor Select(bool Condition, FFontColor A, FFontColor B)
{
	if (Condition)
	{
		return A;
	}
	return B; 
}

// Predefined colors
#define FontWhite		InitFontColor(1, 1, 1)
#define FontGrey		InitFontColor(0.5, 0.5, 0.5)
#define FontDarkGrey	InitFontColor(0.25, 0.25, 0.25)
#define FontBlack		InitFontColor(0, 0, 0)
#define FontRed			InitFontColor(1, 0, 0)
#define FontGreen		InitFontColor(0, 1, 0)
#define FontBlue		InitFontColor(0, 0, 1)
#define FontYellow		InitFontColor(1, 1, 0)
#define FontCyan		InitFontColor(0, 1, 1)
#define FontMagenta		InitFontColor(1, 0, 1)
#define FontOrange		InitFontColor(243.f / 255.f, 156.f / 255.f, 18.f / 255.f)
#define FontPurple		InitFontColor(169.f / 255.f, 7.f / 255.f, 228.f / 255.f)
#define FontTurquoise	InitFontColor(26.f / 255.f, 188.f / 255.f, 156.f / 255.f)
#define FontSilver		InitFontColor(189.f / 255.f, 195.f / 255.f, 199.f / 255.f)
#define FontEmerald		InitFontColor(46.f / 255.f, 204.f / 255.f, 113.f / 255.f)
#define FontLightRed	InitFontColor(0.75f, 0.50f, 0.50f)
#define FontLightGreen	InitFontColor(0.50f, 0.75f, 0.50f)
#define FontLightBlue	InitFontColor(0.50f, 0.50f, 0.75f)
#define FontLightYellow	InitFontColor(0.75f, 0.75f, 0.50f)
#define FontDarkRed		InitFontColor(0.50f, 0.25f, 0.25f)
#define FontDarkGreen	InitFontColor(0.25f, 0.50f, 0.25f)
#define FontDarkBlue	InitFontColor(0.25f, 0.25f, 0.50f)

///////////////////////////////////////////////////////////////////////////////////////////////////
// Primitive colors

#define ColorWhite		float4(1, 1, 1, 1)
#define ColorBlack		float4(0, 0, 0, 1)
#define ColorRed		float4(1, 0, 0, 1)
#define ColorGreen		float4(0, 1, 0, 1)
#define ColorBlue		float4(0, 0, 1, 1)
#define ColorYellow		float4(1, 1, 0, 1)
#define ColorCyan		float4(0, 1, 1, 1)
#define ColorMagenta	float4(1, 0, 1, 1)
#define ColorOrange		float4(243.f / 255.f, 156.f / 255.f,  18.f / 255.f, 1)
#define ColorPurple		float4(169.f / 255.f,   7.f / 255.f, 228.f / 255.f, 1)
#define ColorTurquoise	float4( 26.f / 255.f, 188.f / 255.f, 156.f / 255.f, 1)
#define ColorSilver		float4(189.f / 255.f, 195.f / 255.f, 199.f / 255.f, 1)
#define ColorEmerald	float4( 46.f / 255.f, 204.f / 255.f, 113.f / 255.f, 1)
#define ColorLightRed	float4(0.75f, 0.50f, 0.50f, 1.0f)
#define ColorLightGreen	float4(0.50f, 0.75f, 0.50f, 1.0f)
#define ColorLightBlue	float4(0.50f, 0.50f, 0.75f, 1.0f)
#define ColorDarkRed	float4(0.50f, 0.25f, 0.25f, 1.0f)
#define ColorDarkGreen	float4(0.25f, 0.50f, 0.25f, 1.0f)
#define ColorDarkBlue	float4(0.25f, 0.25f, 0.50f, 1.0f)
#define ColorDarkOrange	float4(ColorOrange.xyz * 0.5f,1.0f)

///////////////////////////////////////////////////////////////////////////////////////////////////
// Util pack/unpack functions

struct FShaderPrintItem
{
	float2 ScreenPos; // Position in normalized coordinates
	int Value;        // Cast to value or symbol
	int Type;         // SHADER_PRINT_TYPE_* defines how to read Value
	float3 Color;	  // Color 
	uint Metadata;	  // Metadata (used for widget to store extra state data
};

// Needs to match C++ code in ShaderPrint.cpp
struct FPackedShaderPrintItem
{
	uint ScreenPos16bits; // Position in normalized coordinates
	uint Value;           // Cast to value or symbol
	uint TypeAndColor;    //
	uint Metadata;	      //
};

FPackedShaderPrintItem PackShaderPrintItem(FShaderPrintItem In)
{
	const uint3 Color8bits = uint3(saturate(In.Color) * 255.0f);
	const uint2 ScreenPos16bit = f32tof16(In.ScreenPos);

	FPackedShaderPrintItem Out;
	Out.ScreenPos16bits = ScreenPos16bit.x | (ScreenPos16bit.y<<16);
	Out.Value = asuint(In.Value);
	Out.TypeAndColor = (Color8bits.z << 24) | (Color8bits.y << 16) | (Color8bits.x << 8) | (In.Type & 0xFF);
	Out.Metadata = In.Metadata;
	return Out;
}

FShaderPrintItem UnpackShaderPrintItem(FPackedShaderPrintItem In)
{
	const uint2 ScreenPos16bits = uint2(In.ScreenPos16bits & 0xFFFF, (In.ScreenPos16bits>>16) & 0xFFFF);

	FShaderPrintItem Out;
	Out.ScreenPos = f16tof32(ScreenPos16bits);
	Out.Value	  = asint(In.Value);
	Out.Type	  = (In.TypeAndColor) & 0xFF;
	Out.Color.x   = float((In.TypeAndColor >>  8) & 0xFF) / 255.f;
	Out.Color.y   = float((In.TypeAndColor >> 16) & 0xFF) / 255.f;
	Out.Color.z   = float((In.TypeAndColor >> 24) & 0xFF) / 255.f;
	Out.Metadata  = In.Metadata;
	return Out;
}

// /!\ If you update this function, update the WriteSymbol() version without explicit buffer, below in this file
void WriteSymbol(uint Offset, FShaderPrintItem In, RWBuffer<uint> InRWBuffer)
{
	const uint Offset4 = GetSymbolOffset(Offset);

	const FPackedShaderPrintItem Packed = PackShaderPrintItem(In);
	InRWBuffer[Offset4 + 0] = Packed.ScreenPos16bits;
	InRWBuffer[Offset4 + 1] = Packed.Value;
	InRWBuffer[Offset4 + 2] = Packed.TypeAndColor;
	InRWBuffer[Offset4 + 3] = Packed.Metadata;
}

FShaderPrintItem ReadSymbol(uint Offset, StructuredBuffer<uint> InBuffer)
{
	const uint Offset4 = GetSymbolOffset(Offset);

	FPackedShaderPrintItem Packed = (FPackedShaderPrintItem)0;
	Packed.ScreenPos16bits	= InBuffer[Offset4 + 0];
	Packed.Value			= InBuffer[Offset4 + 1];
	Packed.TypeAndColor		= InBuffer[Offset4 + 2];
	Packed.Metadata			= InBuffer[Offset4 + 3];

	return UnpackShaderPrintItem(Packed);
}

FShaderPrintItem ReadSymbol(uint Offset, Buffer<uint> InBuffer)
{
	const uint Offset4 = GetSymbolOffset(Offset);

	FPackedShaderPrintItem Packed = (FPackedShaderPrintItem)0;
	Packed.ScreenPos16bits	= InBuffer[Offset4 + 0];
	Packed.Value			= InBuffer[Offset4 + 1];
	Packed.TypeAndColor		= InBuffer[Offset4 + 2];
	Packed.Metadata			= InBuffer[Offset4 + 3];

	return UnpackShaderPrintItem(Packed);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Context structures.
// Stores the current dynamic state, and the constant data describing config and buffer bindings.

struct FShaderPrintConfig
{
	int2 Resolution;
	int2 CursorCoord;
	float3 TranslatedWorldOffset;
	float2 FontSize;
	float2 FontSpacing;
	uint MaxCharacterCount;
	uint MaxSymbolCount;
	uint MaxStateCount;
	uint MaxLineCount;
	uint MaxTriangleCount;
	bool bIsDrawLocked;
};

struct FShaderPrintBuffers
{
	Buffer<uint> StateBuffer;
	RWBuffer<uint> RWEntryBuffer;
};

struct FShaderPrintContext
{
	bool bIsActive;
	float2 StartPos;
	float2 Pos;

	// Text rectangle, useful for drawing text background
	float2 RectMin;
	float2 RectMax;

	FShaderPrintConfig Config;

	bool IsDrawLocked() { return Config.bIsDrawLocked; }

#if !SHADER_PRINT_USE_GLOBAL_RESOURCE
	FShaderPrintBuffers Buffers;
#endif
};

// Version using SHADER_PRINT_RWENTRYBUFFER in order to provide offset value
void WriteSymbol(uint Offset, FShaderPrintItem In, FShaderPrintContext Ctx) 
{
	const uint Offset4 = GetSymbolOffset(Offset);

	const FPackedShaderPrintItem Packed = PackShaderPrintItem(In);
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Offset4 + 0) = Packed.ScreenPos16bits;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Offset4 + 1) = Packed.Value;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Offset4 + 2) = Packed.TypeAndColor;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Offset4 + 3) = Packed.Metadata;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Internal helpers

void ShaderPrint_Internal(FShaderPrintContext Ctx, in FShaderPrintItem Item)
{
	// If MaxCharacterCount is 0 then we don't reset the buffer counter so early out here
	if (!Ctx.bIsActive || Ctx.Config.MaxCharacterCount == 0)
	{
		return;
	}

	// Buffer counter is stored in first element .Value
	int IndexToStore = 0;
	SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(Ctx,SHADER_PRINT_COUNTER_OFFSET_SYMBOL), 1, IndexToStore);

	// Prevent writing off the buffer
	// Note that counter still increases so need clamp when reading it in later passes
	if (uint(IndexToStore) >= Ctx.Config.MaxCharacterCount)
	{
		return;
	}

	WriteSymbol(IndexToStore, Item, Ctx);
}

void ShaderPrint_Internal(FShaderPrintContext Ctx, in float2 ScreenPos, in int Value, in FFontColor FontColor, in uint Metadata, in int Type)
{
	FShaderPrintItem Item;
	Item.ScreenPos = ScreenPos;
	Item.Value = Value;
	Item.Type = Type;
	Item.Color = FontColor.Color;
	Item.Metadata = Metadata;
	ShaderPrint_Internal(Ctx, Item);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Symbol printing

void PrintSymbol(inout FShaderPrintContext Ctx, in int Symbol, in FFontColor Color)
{
	if (Ctx.bIsActive)
	{
		ShaderPrint_Internal(Ctx, Ctx.Pos, Symbol, Color, 0u, SHADER_PRINT_TYPE_SYMBOL);
		Ctx.Pos.x += Ctx.Config.FontSpacing.x;
	}
}
void PrintSymbol(inout FShaderPrintContext Ctx, in int Symbol) { PrintSymbol(Ctx, Symbol, GetDefaultFontColor()); }

void Newline(inout FShaderPrintContext Ctx)
{
	if (Ctx.bIsActive)
	{
		Ctx.RectMax.x = max(Ctx.RectMax.x, Ctx.Pos.x);
		Ctx.RectMax.y = Ctx.Pos.y + Ctx.Config.FontSpacing.y;

		Ctx.Pos.x = Ctx.StartPos.x;
		Ctx.Pos.y += Ctx.Config.FontSpacing.y;
	}
}

int2 GetCursorPos(FShaderPrintContext Ctx)
{
	return Ctx.Config.CursorCoord;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Text & hash

struct FShaderPrintText
{
	uint Index;
};

FShaderPrintText InitShaderPrintText(uint InIndex)
{
	FShaderPrintText Out;
	Out.Index = InIndex;
	return Out;
}

// Forward declarations for text accessors
uint ShaderPrintGetChar(uint InIndex);
uint ShaderPrintGetOffset(FShaderPrintText InTextEntry);
uint ShaderPrintGetHash(FShaderPrintText InTextEntry);
 
// Function for reading global TEXT string
ISOLATE
float2 ShaderPrintText_Internal(FShaderPrintContext Ctx, bool bIsActive, float2 InPos, FShaderPrintText InTextEntry, FFontColor InColor)
{
	// If MaxCharacterCount is 0 then we don't reset the buffer counter so early out here
	if (bIsActive && Ctx.Config.MaxCharacterCount > 0)
	{
		const uint Begin = ShaderPrintGetOffset(InTextEntry);
		const uint End   = ShaderPrintGetOffset(InitShaderPrintText(InTextEntry.Index + 1));
		const uint Count = End - Begin;

		// Buffer counter is stored in first element .Value
		// Prevent writing off the buffer
		// Note that counter still increases so need clamp when reading it in later passes
		int IndexToStore = 0;
		SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(Ctx,SHADER_PRINT_COUNTER_OFFSET_SYMBOL), Count, IndexToStore);
		if (uint(IndexToStore + Count) < Ctx.Config.MaxCharacterCount)
		{
			for (uint i = Begin; i < End; ++i)
			{
				FShaderPrintItem Item;
				Item.ScreenPos = InPos;
				Item.Value = ShaderPrintGetChar(i);
				Item.Type = SHADER_PRINT_TYPE_SYMBOL;
				Item.Color = InColor.Color;
				Item.Metadata = 0u;

				WriteSymbol(IndexToStore, Item, Ctx);

				++IndexToStore;
				InPos.x += Ctx.Config.FontSpacing.x;
			}	
		}
	}
	return InPos;
}

void Print(inout FShaderPrintContext Ctx, FShaderPrintText InTextEntry, FFontColor InColor)	{ Ctx.Pos = ShaderPrintText_Internal(Ctx, Ctx.bIsActive, Ctx.Pos, InTextEntry, InColor); }
void Print(inout FShaderPrintContext Ctx, FShaderPrintText InTextEntry)						{ Ctx.Pos = ShaderPrintText_Internal(Ctx, Ctx.bIsActive, Ctx.Pos, InTextEntry, GetDefaultFontColor()); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Value printing (common value printing)
#define MAX_DECIMAL_COUNT 5u
#define MAX_DIGIT_COUNT 12u

float2 ShaderPrintValue_Internal(FShaderPrintContext Ctx, float2 ScreenPos, float Value, FFontColor Color, uint MaxDigit, uint MaxDecimal)  { ShaderPrint_Internal(Ctx, ScreenPos, asint(Value),           Color, MaxDecimal, SHADER_PRINT_TYPE_FLOAT); ScreenPos.x += Ctx.Config.FontSpacing.x * MaxDigit; return ScreenPos; }
float2 ShaderPrintValue_Internal(FShaderPrintContext Ctx, float2 ScreenPos, int   Value, FFontColor Color, uint MaxDigit, uint MaxDecimal)  { ShaderPrint_Internal(Ctx, ScreenPos, Value,                  Color, MaxDecimal, SHADER_PRINT_TYPE_INT);   ScreenPos.x += Ctx.Config.FontSpacing.x * MaxDigit; return ScreenPos; }
float2 ShaderPrintValue_Internal(FShaderPrintContext Ctx, float2 ScreenPos, uint  Value, FFontColor Color, uint MaxDigit, uint MaxDecimal)  { ShaderPrint_Internal(Ctx, ScreenPos, asint(Value),           Color, MaxDecimal, SHADER_PRINT_TYPE_UINT);  ScreenPos.x += Ctx.Config.FontSpacing.x * MaxDigit; return ScreenPos; }
float2 ShaderPrintValue_Internal(FShaderPrintContext Ctx, float2 ScreenPos, bool  Value, FFontColor Color, uint MaxDigit, uint MaxDecimal)  { ShaderPrint_Internal(Ctx, ScreenPos, asint(Value ? 1u : 0u), Color, MaxDecimal, SHADER_PRINT_TYPE_UINT);  ScreenPos.x += Ctx.Config.FontSpacing.x * MaxDigit; return ScreenPos; }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Scalar type printing

#define SHADER_PRINT_OVERLOAD_1(InNumComponent, InType) \
void   Print(inout FShaderPrintContext Ctx, InType Value, FFontColor Color, uint MaxDigit=MAX_DIGIT_COUNT, uint MaxDecimal=MAX_DECIMAL_COUNT)	{ if (Ctx.bIsActive) { Ctx.Pos = ShaderPrintValue_Internal(Ctx, Ctx.Pos, Value, Color, MaxDigit, MaxDecimal); } } \
void   Print(inout FShaderPrintContext Ctx, InType Value)																						{ if (Ctx.bIsActive) { Ctx.Pos = ShaderPrintValue_Internal(Ctx, Ctx.Pos, Value, GetDefaultFontColor(), MAX_DIGIT_COUNT, MAX_DECIMAL_COUNT); } }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Vector type printing

#define SHADER_PRINT_OVERLOAD_N(InNumComponent, InType) \
void   Print(inout FShaderPrintContext Ctx, InType Value, FFontColor Color, uint MaxDigit=MAX_DIGIT_COUNT, uint MaxDecimal=MAX_DECIMAL_COUNT)	{ if (Ctx.bIsActive) { UNROLL for (uint CompIt=0;CompIt<InNumComponent;++CompIt) { Ctx.Pos = ShaderPrintValue_Internal(Ctx, Ctx.Pos, Value[CompIt], Color, MaxDigit, MaxDecimal); } } } \
void   Print(inout FShaderPrintContext Ctx, InType Value)																						{ if (Ctx.bIsActive) { UNROLL for (uint CompIt=0;CompIt<InNumComponent;++CompIt) { Ctx.Pos = ShaderPrintValue_Internal(Ctx, Ctx.Pos, Value[CompIt], GetDefaultFontColor(), MAX_DIGIT_COUNT, MAX_DECIMAL_COUNT); } } }

// Print(X) version
SHADER_PRINT_OVERLOAD_1(1, float)
SHADER_PRINT_OVERLOAD_N(2, float2)
SHADER_PRINT_OVERLOAD_N(3, float3)
SHADER_PRINT_OVERLOAD_N(4, float4)

SHADER_PRINT_OVERLOAD_1(1, uint)
SHADER_PRINT_OVERLOAD_N(2, uint2)
SHADER_PRINT_OVERLOAD_N(3, uint3)
SHADER_PRINT_OVERLOAD_N(4, uint4)

SHADER_PRINT_OVERLOAD_1(1, int)
SHADER_PRINT_OVERLOAD_N(2, int2)
SHADER_PRINT_OVERLOAD_N(3, int3)
SHADER_PRINT_OVERLOAD_N(4, int4)

SHADER_PRINT_OVERLOAD_1(1, bool)

///////////////////////////////////////////////////////////////////////////////////////////////////
// Matrix type printing

#define SHADER_PRINT_OVERLOAD_NN(InNumComponentX, InNumComponentY, InType) \
void   Print(inout FShaderPrintContext Ctx, InType Value, FFontColor Color, uint MaxDigit=MAX_DIGIT_COUNT, uint MaxDecimal=MAX_DECIMAL_COUNT)	{ if (Ctx.bIsActive) { UNROLL for (uint CompY=0;CompY<InNumComponentY;++CompY) { Print(Ctx, Value[CompY], Color, MaxDigit, MaxDecimal); Ctx.Pos += float2(0.f, Ctx.Config.FontSpacing.y); } } } \
void   Print(inout FShaderPrintContext Ctx, InType Value)																						{ if (Ctx.bIsActive) { UNROLL for (uint CompY=0;CompY<InNumComponentY;++CompY) { Print(Ctx, Value[CompY], GetDefaultFontColor(), MAX_DIGIT_COUNT, MAX_DECIMAL_COUNT); Ctx.Pos += float2(0.f, Ctx.Config.FontSpacing.y); } } }

// Print(X) version
SHADER_PRINT_OVERLOAD_NN(3, 3, float3x3)
SHADER_PRINT_OVERLOAD_NN(4, 3, float4x3)
SHADER_PRINT_OVERLOAD_NN(4, 4, float4x4)

///////////////////////////////////////////////////////////////////////////////////////////////////
// Bool printing

// Note: Use a define rather than a function to avoid unecessary TEXT() parsing during shader compilation, even for shader not using ShaderPrint.
#define PrintBool(Ctx, In) { if (In) Print(Ctx, TEXT("Yes "), FontGreen); else Print(Ctx, TEXT("No  "), FontRed); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Widgets (experimental)

uint2 GetCheckboxWidgetSize(float2 FontSpacing, int2 Resolution)
{
	return uint(FontSpacing.x * Resolution.x).xx;
}

uint2 GetSliderWidgetSize(float2 FontSpacing, int2 Resolution)
{
	return uint2(10,1) * uint(FontSpacing.x * Resolution.x);
}

struct FShaderPrintMetadata
{
	uint Hash;
	uint Index;
};

uint ShaderPrintPackMetadata(uint InHash, uint InIndex)
{
	return ((InIndex & 0xFFu) << 24) | (InHash & SHADER_PRINT_STATE_HASH_MASK);
}
uint ShaderPrintPackMetadata(FShaderPrintMetadata In) { return ShaderPrintPackMetadata(In.Hash, In.Index); }

FShaderPrintMetadata ShaderPrintUnpackMetadata(uint Metadata)
{
	FShaderPrintMetadata Out;
	Out.Hash = (Metadata & SHADER_PRINT_STATE_HASH_MASK);
	Out.Index= (Metadata >> 24) & 0xFF;
	return Out;
}

// Return true if found, false otherwise
bool ShaderPrintGetStateValue(FShaderPrintContext Ctx, in uint InHash, inout uint OutIndex, inout uint OutValue)
{
	OutIndex = SHADER_PRINT_STATE_INVALID_INDEX;
	OutValue = 0;

	// Slow linear search among all widget states
	const uint MaxCount = min(SHADER_PRINT_RWSTATEBUFFER(Ctx, SHADER_PRINT_STATE_COUNT_OFFSET), Ctx.Config.MaxStateCount);
	for (uint Index = 0; Index < MaxCount; ++Index)
	{
		const uint Index3 = Index * SHADER_PRINT_STATE_STRIDE + SHADER_PRINT_STATE_VALUE_OFFSET;
		const FShaderPrintMetadata Metadata = ShaderPrintUnpackMetadata(SHADER_PRINT_RWSTATEBUFFER(Ctx, Index3 + SHADER_PRINT_STATE_INDEX_METADATA));
		if (Metadata.Hash == InHash)
		{
			OutIndex = Metadata.Index;
			OutValue = SHADER_PRINT_RWSTATEBUFFER(Ctx, Index3 + SHADER_PRINT_STATE_INDEX_VALUE);
			return true;
		}
	}
	return false;
}

// Add a checkbox widget 
bool AddCheckbox(inout FShaderPrintContext Ctx, FShaderPrintText InTextEntry, bool bDefault, in FFontColor InColor)
{
	uint Index = 0;
	uint RawValue = 0;
	const uint Hash = ShaderPrintGetHash(InTextEntry) & SHADER_PRINT_STATE_HASH_MASK; // Trunk hask since metadata has limited storage
	const bool bIsValid = ShaderPrintGetStateValue(Ctx, Hash, Index, RawValue);
	RawValue = bIsValid ? RawValue : (bDefault ? 0x1 : 0x0);

	if (Ctx.bIsActive)
	{
		const uint2 Coord = Ctx.Pos * Ctx.Config.Resolution;
		const uint2 Cusor = Ctx.Config.CursorCoord;
		const uint2 WidgetSize = GetCheckboxWidgetSize(Ctx.Config.FontSpacing, Ctx.Config.Resolution);
		const uint2 WidgetMax = uint2(Coord.x + WidgetSize.x, Coord.y + WidgetSize.y / 2);
		const uint2 WidgetMin = uint2(Coord.x, Coord.y - WidgetSize.y / 2);
		const bool  bIsInside = all(Cusor >= WidgetMin) && all(Cusor <= WidgetMax);

		const bool bWasInside = RawValue & 0x2;

		// Update checkbox value 
		// Track if the cursor was already within the box or not
		if (bIsInside)
		{
			if (!bWasInside)
			{
				// First time the cursor is within the checkbox
				RawValue = ((RawValue & 0x1) == 1 ? 0x0 : 0x1) | 0x2;
			}
		}
		else
		{
			RawValue = (RawValue & 0x1);
		}

		FShaderPrintItem E;
		E.ScreenPos = Ctx.Pos;
		E.Value = RawValue;
		E.Type = SHADER_PRINT_TYPE_CHECK;
		E.Color = bIsInside ? FontYellow.Color : FontWhite.Color;// InColor.Color;
		E.Metadata = ShaderPrintPackMetadata(Hash, Index);
		ShaderPrint_Internal(Ctx, E);

		Ctx.Pos.x += float(WidgetSize.x) / float(Ctx.Config.Resolution.x) + Ctx.Config.FontSpacing.x;

		// Text
		Print(Ctx, InTextEntry, InColor);
		Ctx.Pos.x += Ctx.Config.FontSpacing.x;
	}

	return RawValue & 0x1;
}

// Add a slider widget  
float AddSlider(inout FShaderPrintContext Ctx, FShaderPrintText InTextEntry, float bDefault, in FFontColor InColor, float InMin = 0.f, float InMax = 1.f)
{
	// The value is stored as normalized, for displaying slide value correctly. This is not idealy
	// as it implies some precision lost due to back&forth conversion between normalized/non-normalized value.
	uint Index = 0;
	uint RawValue = 0;
	const uint Hash = ShaderPrintGetHash(InTextEntry) & SHADER_PRINT_STATE_HASH_MASK; // Trunk hask since metadata has limited storage
	const bool bIsValid = ShaderPrintGetStateValue(Ctx, Hash, Index, RawValue);
	float NormalizedValue = bIsValid ? asfloat(RawValue) : bDefault; 
	float Value = NormalizedValue * (InMax - InMin) + InMin;

	if (Ctx.bIsActive)
	{
		const uint2 Coord = Ctx.Pos * Ctx.Config.Resolution;
		const uint2 Cusor = Ctx.Config.CursorCoord;
		const uint2 WidgetSize = GetSliderWidgetSize(Ctx.Config.FontSpacing, Ctx.Config.Resolution);
		const uint2 WidgetMax = uint2(Coord.x + WidgetSize.x, Coord.y + WidgetSize.y / 2);
		const uint2 WidgetMin = uint2(Coord.x, Coord.y - WidgetSize.y / 2);
		const bool  bIsInside = all(Cusor >= WidgetMin) && all(Cusor <= WidgetMax);

		// Update slider value
		if (bIsInside)
		{
			const float S = saturate(float(Cusor.x - WidgetMin.x) / float(WidgetMax.x - WidgetMin.x));
			Value = lerp(InMin, InMax, S);
		}

		NormalizedValue = saturate((Value - InMin) / (InMax - InMin));

		FShaderPrintItem E;
		E.ScreenPos = Ctx.Pos;
		E.Value = asint(NormalizedValue);
		E.Type = SHADER_PRINT_TYPE_SLIDER;
		E.Color = bIsInside ? FontYellow.Color : FontWhite.Color; // InColor.Color;
		E.Metadata = ShaderPrintPackMetadata(Hash, Index);
		ShaderPrint_Internal(Ctx, E);

		Ctx.Pos.x += float(WidgetSize.x) / float(Ctx.Config.Resolution.x) + Ctx.Config.FontSpacing.x;

		// Text
		Print(Ctx, InTextEntry, InColor);
		Ctx.Pos.x += Ctx.Config.FontSpacing.x;
	}

	return Value;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Primitives types

///////////////////////////////////////////////////////////////////////////////////////////////////
// Triangle-based Primitives (Triangle, Point, ...)

struct FTriangleElement 
{
	float3 Pos0;
	float3 Pos1;
	float3 Pos2;
	float4 Color;
	bool bIsScreenSpace;
};

struct FPackedTriangleElement
{
	uint4 Packed0;
	uint4 Packed1;
	uint4 Packed2;
};

FTriangleElement UnpackTriangleElement(FPackedTriangleElement In)
{
	FTriangleElement Out = (FTriangleElement)0;
	{
		Out.Pos0.x = asfloat(In.Packed0.x);
		Out.Pos0.y = asfloat(In.Packed0.y);
		Out.Pos0.z = asfloat(In.Packed0.z);
		Out.Color  = float4((In.Packed0.w >> 24) & 0xFF, (In.Packed0.w >> 16) & 0xFF, (In.Packed0.w >> 8) & 0xFF, (In.Packed0.w) & 0xFF) / 255.0f;
	}
	{
		Out.Pos1.x = asfloat(In.Packed1.x);
		Out.Pos1.y = asfloat(In.Packed1.y);
		Out.Pos1.z = asfloat(In.Packed1.z);

		// The 'space' info is stored into the alpha's LSB
		Out.bIsScreenSpace = (In.Packed1.w & 0x1) > 0u;
	}
	{
		Out.Pos2.x = asfloat(In.Packed2.x);
		Out.Pos2.y = asfloat(In.Packed2.y);
		Out.Pos2.z = asfloat(In.Packed2.z);
	}
	return Out;
}

FPackedTriangleElement PackTriangleElement(FTriangleElement In)
{
	uint4 PackedC = uint4(255.0f * saturate(In.Color));

	FPackedTriangleElement Out = (FPackedTriangleElement)0;
	Out.Packed0.x = asuint(In.Pos0.x);
	Out.Packed0.y = asuint(In.Pos0.y);
	Out.Packed0.z = asuint(In.Pos0.z);
	Out.Packed0.w = (PackedC.x << 24) | (PackedC.y << 16) | (PackedC.z << 8) | (PackedC.w);

	Out.Packed1.x = asuint(In.Pos1.x);
	Out.Packed1.y = asuint(In.Pos1.y);
	Out.Packed1.z = asuint(In.Pos1.z);
	Out.Packed1.w = In.bIsScreenSpace ? 0x1 : 0x0;

	Out.Packed2.x = asuint(In.Pos2.x);
	Out.Packed2.y = asuint(In.Pos2.y);
	Out.Packed2.z = asuint(In.Pos2.z);
	Out.Packed2.w = 0;
	return Out;
}

FTriangleElement UnpackTriangleElement(Buffer<uint> InPrimitiveBuffer, uint InIndex, uint MaxCharacterCount, uint MaxLineCount)
{
	const uint Index12 = GetPrimitiveTriangleOffset(InIndex, MaxCharacterCount, MaxLineCount);

	FPackedTriangleElement Out = (FPackedTriangleElement)0;
	Out.Packed0.x = InPrimitiveBuffer[Index12 + 0];
	Out.Packed0.y = InPrimitiveBuffer[Index12 + 1];
	Out.Packed0.z = InPrimitiveBuffer[Index12 + 2];
	Out.Packed0.w = InPrimitiveBuffer[Index12 + 3];

	Out.Packed1.x = InPrimitiveBuffer[Index12 + 4];
	Out.Packed1.y = InPrimitiveBuffer[Index12 + 5];
	Out.Packed1.z = InPrimitiveBuffer[Index12 + 6];
	Out.Packed1.w = InPrimitiveBuffer[Index12 + 7];

	Out.Packed2.x = InPrimitiveBuffer[Index12 + 8];
	Out.Packed2.y = InPrimitiveBuffer[Index12 + 9];
	Out.Packed2.z = InPrimitiveBuffer[Index12 + 10];
	Out.Packed2.w = InPrimitiveBuffer[Index12 + 11];
	return UnpackTriangleElement(Out);
}

bool AllocateTriangleElement(FShaderPrintContext Ctx, uint Count, inout uint OutIndex)
{
	OutIndex = 0;
	if (Ctx.Config.MaxTriangleCount == 0)
	{
		return false;
	}

	SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(Ctx,SHADER_PRINT_COUNTER_OFFSET_TRIANGLE), Count, OutIndex);
	return (OutIndex + Count) < Ctx.Config.MaxTriangleCount;
}

void AddTriangleElement(FShaderPrintContext Ctx, FTriangleElement In, uint Index)
{
	const uint Index12 = GetPrimitiveTriangleOffset(Index, Ctx.Config.MaxCharacterCount, Ctx.Config.MaxLineCount);

	FPackedTriangleElement Out = PackTriangleElement(In);
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 0) = Out.Packed0.x;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 1) = Out.Packed0.y;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 2) = Out.Packed0.z;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 3) = Out.Packed0.w;

	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 4) = Out.Packed1.x;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 5) = Out.Packed1.y;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 6) = Out.Packed1.z;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 7) = Out.Packed1.w;

	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 8) = Out.Packed2.x;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 9) = Out.Packed2.y;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 10) = Out.Packed2.z;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, Index12 + 11) = Out.Packed2.w;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Filled Triangle
void AddFilledTriangleTWS(FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float4 Color)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateTriangleElement(Ctx, 1, Offset))
		{
			FTriangleElement Element;
			Element.bIsScreenSpace = false;
			Element.Color = Color;

			Element.Pos0 = Pos0;
			Element.Pos1 = Pos1;
			Element.Pos2 = Pos2;
			AddTriangleElement(Ctx, Element, Offset + 0);
		}
	}
}

void AddFilledTriangleWS (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float4 Color) { AddFilledTriangleTWS(Ctx, Pos0 + Ctx.Config.TranslatedWorldOffset, Pos1 + Ctx.Config.TranslatedWorldOffset, Pos2 + Ctx.Config.TranslatedWorldOffset, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Filled Quad

void AddFilledQuadTWS(FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateTriangleElement(Ctx, 2, Offset))
		{
			FTriangleElement Element;
			Element.bIsScreenSpace = false;
			Element.Color = Color;

			Element.Pos0 = Pos0;
			Element.Pos1 = Pos1;
			Element.Pos2 = Pos2;
			AddTriangleElement(Ctx, Element, Offset + 0);

			Element.Pos0 = Pos0;
			Element.Pos1 = Pos2;
			Element.Pos2 = Pos3;
			AddTriangleElement(Ctx, Element, Offset + 1);
		}
	}
}

void AddFilledQuadWS (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color) { AddFilledQuadTWS(Ctx, Pos0 + Ctx.Config.TranslatedWorldOffset, Pos1 + Ctx.Config.TranslatedWorldOffset, Pos2 + Ctx.Config.TranslatedWorldOffset, Pos3 + Ctx.Config.TranslatedWorldOffset, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Filled Quad (Screen-Space)

void AddFilledQuadSS(FShaderPrintContext Ctx, float2 InPosInPixel0, float2 InPosInPixel2, float4 Color)
{
	if (Ctx.bIsActive)
	{
		const float2 InPosInPixel1 = float2(InPosInPixel2.x, InPosInPixel0.y);
		const float2 InPosInPixel3 = float2(InPosInPixel0.x, InPosInPixel2.y);

		const float Depth = 0.5f;
		const float3 Pos0 = float3((InPosInPixel0) / float2(Ctx.Config.Resolution), Depth);
		const float3 Pos1 = float3((InPosInPixel1) / float2(Ctx.Config.Resolution), Depth);
		const float3 Pos2 = float3((InPosInPixel2) / float2(Ctx.Config.Resolution), Depth);
		const float3 Pos3 = float3((InPosInPixel3) / float2(Ctx.Config.Resolution), Depth);

		uint Offset = 0;
		if (AllocateTriangleElement(Ctx, 2, Offset))
		{
			FTriangleElement Element;
			Element.bIsScreenSpace = true;
			Element.Color = Color;

			Element.Pos0 = Pos0;
			Element.Pos1 = Pos1;
			Element.Pos2 = Pos2;
			AddTriangleElement(Ctx, Element, Offset + 0);

			Element.Pos0 = Pos0;
			Element.Pos1 = Pos2;
			Element.Pos2 = Pos3;
			AddTriangleElement(Ctx, Element, Offset + 1);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Line-based Primitives (Line, AABB, OBB, Referentials, ...)

struct FLineElement 
{
	float3 Pos0;
	float3 Pos1;
	float4 Color0;
	float4 Color1;
	bool bIsScreenSpace;
};

struct FPackedLineElement
{
	uint4 Packed0;
	uint4 Packed1;
};

FLineElement UnpackLineElement(FPackedLineElement In)
{
	FLineElement Out = (FLineElement)0;
	{
		Out.Pos0.x = asfloat(In.Packed0.x);
		Out.Pos0.y = asfloat(In.Packed0.y);
		Out.Pos0.z = asfloat(In.Packed0.z);
		Out.Color0 = float4((In.Packed0.w >> 24) & 0xFF, (In.Packed0.w >> 16) & 0xFF, (In.Packed0.w >> 8) & 0xFF, (In.Packed0.w) & 0xFF) / 255.0f;
	}
	{
		Out.Pos1.x = asfloat(In.Packed1.x);
		Out.Pos1.y = asfloat(In.Packed1.y);
		Out.Pos1.z = asfloat(In.Packed1.z);
		Out.Color1 = float4((In.Packed1.w >> 24) & 0xFF, (In.Packed1.w >> 16) & 0xFF, (In.Packed1.w >> 8) & 0xFF, (In.Packed1.w) & 0xFF) / 255.0f;

		// The 'space' info is stored into the alpha's LSB
		Out.bIsScreenSpace = (In.Packed1.w & 0x1) > 0u;
	}
	return Out;
}

FPackedLineElement PackLineElement(FLineElement In)
{
	uint4 PackedC0 = uint4(255.0f * saturate(In.Color0));
	uint4 PackedC1 = uint4(255.0f * saturate(In.Color1));

	// Store 'space' info into the alpha's LSB
	PackedC1 = PackedC1 & 0xFE;
	PackedC1 = PackedC1 | (In.bIsScreenSpace ? 0x1 : 0x0);

	FPackedLineElement Out = (FPackedLineElement)0;
	Out.Packed0.x = asuint(In.Pos0.x);
	Out.Packed0.y = asuint(In.Pos0.y);
	Out.Packed0.z = asuint(In.Pos0.z);
	Out.Packed0.w = (PackedC0.x << 24) | (PackedC0.y << 16) | (PackedC0.z << 8) | (PackedC0.w);

	Out.Packed1.x = asuint(In.Pos1.x);
	Out.Packed1.y = asuint(In.Pos1.y);
	Out.Packed1.z = asuint(In.Pos1.z);
	Out.Packed1.w = (PackedC1.x << 24) | (PackedC1.y << 16) | (PackedC1.z << 8) | (PackedC1.w);
	return Out;
}

FLineElement UnpackLineElement(Buffer<uint> InPrimitiveBuffer, uint InIndex, uint MaxCharacterCount)
{
	const uint Index8 = GetPrimitiveLineOffset(InIndex, MaxCharacterCount);

	FPackedLineElement Out = (FPackedLineElement)0;
	Out.Packed0.x = InPrimitiveBuffer[Index8 + 0];
	Out.Packed0.y = InPrimitiveBuffer[Index8 + 1];
	Out.Packed0.z = InPrimitiveBuffer[Index8 + 2];
	Out.Packed0.w = InPrimitiveBuffer[Index8 + 3];

	Out.Packed1.x = InPrimitiveBuffer[Index8 + 4];
	Out.Packed1.y = InPrimitiveBuffer[Index8 + 5];
	Out.Packed1.z = InPrimitiveBuffer[Index8 + 6];
	Out.Packed1.w = InPrimitiveBuffer[Index8 + 7];
	return UnpackLineElement(Out);
}

bool AllocateLineElement(FShaderPrintContext Ctx, uint Count, inout uint OutIndex)
{
	OutIndex = 0;
	if (Ctx.Config.MaxLineCount == 0)
	{
		return false;
	}

	SHADER_PRINT_INTERLOCKEDADD(SHADER_PRINT_RWENTRYBUFFER(Ctx,SHADER_PRINT_COUNTER_OFFSET_LINE), Count, OutIndex);
	return (OutIndex + Count) < Ctx.Config.MaxLineCount;
}

void AddLineElement(FShaderPrintContext Ctx, FLineElement In, uint Index)
{
	const uint Index8 = GetPrimitiveLineOffset(Index, Ctx.Config.MaxCharacterCount);

	FPackedLineElement Out = PackLineElement(In);
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 0) = Out.Packed0.x;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 1) = Out.Packed0.y;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 2) = Out.Packed0.z;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 3) = Out.Packed0.w;

	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 4) = Out.Packed1.x;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 5) = Out.Packed1.y;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 6) = Out.Packed1.z;
	SHADER_PRINT_RWENTRYBUFFER(Ctx,Index8 + 7) = Out.Packed1.w;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Line

void AddLineTWS(FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color0, float4 Color1)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateLineElement(Ctx, 1, Offset))
		{
			FLineElement Element;
			Element.Pos0 = Pos0;
			Element.Pos1 = Pos1;
			Element.Color0 = Color0;
			Element.Color1 = Color1;
			Element.bIsScreenSpace = false;
			AddLineElement(Ctx, Element, Offset);
		}
	}
}

void AddLineTWS(FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color)					{ AddLineTWS(Ctx, Pos0, Pos1, Color, Color); }
void AddLineWS (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color)					{ AddLineTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Color, Color); }
void AddLineWS (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color0, float4 Color1)	{ AddLineTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Color0, Color1); }
void AddLine   (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color)					{ AddLineTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Color, Color); }
void AddLine   (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float4 Color0, float4 Color1)	{ AddLineTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Color0, Color1); }

// Draw 2D line. Input positions are expressed in pixel coordinate
void AddLineSS(FShaderPrintContext Ctx, float2 InPosInPixel0, float2 InPosInPixel1, float4 Color0, float4 Color1)
{
	if (Ctx.bIsActive) 
	{
		const float2 Pos0 = float2(InPosInPixel0) / float2(Ctx.Config.Resolution);
		const float2 Pos1 = float2(InPosInPixel1) / float2(Ctx.Config.Resolution);

		uint Offset = 0;
		if (AllocateLineElement(Ctx, 1, Offset))
		{
			FLineElement Element;
			Element.Pos0 = float3(Pos0, 0);
			Element.Pos1 = float3(Pos1, 0);
			Element.Color0 = Color0;
			Element.Color1 = Color1;
			Element.bIsScreenSpace = true;
			AddLineElement(Ctx, Element, Offset);
		}
	}
}

void AddLineSS(FShaderPrintContext Ctx, float2 Pos0, float2 Pos1, float4 Color)					{ AddLineSS(Ctx, Pos0, Pos1, Color, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Quad

void AddQuadTWS(FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateLineElement(Ctx, 4, Offset))
		{
			FLineElement Element;
			Element.bIsScreenSpace = false;
			Element.Color0 = Element.Color1 = Color;
			Element.Pos0 = Pos0; Element.Pos1 = Pos1; AddLineElement(Ctx, Element, Offset + 0);
			Element.Pos0 = Pos1; Element.Pos1 = Pos2; AddLineElement(Ctx, Element, Offset + 1);
			Element.Pos0 = Pos2; Element.Pos1 = Pos3; AddLineElement(Ctx, Element, Offset + 2);
			Element.Pos0 = Pos3; Element.Pos1 = Pos0; AddLineElement(Ctx, Element, Offset + 3);
		}
	}
}

void AddQuadWS (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color)	{ AddQuadTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Pos2+Ctx.Config.TranslatedWorldOffset, Pos3+Ctx.Config.TranslatedWorldOffset, Color); }
void AddQuad   (FShaderPrintContext Ctx, float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color)	{ AddQuadTWS(Ctx, Pos0+Ctx.Config.TranslatedWorldOffset, Pos1+Ctx.Config.TranslatedWorldOffset, Pos2+Ctx.Config.TranslatedWorldOffset, Pos3+Ctx.Config.TranslatedWorldOffset, Color); }

// Draw 2D quad line
void AddQuadSS(FShaderPrintContext Ctx, float2 MinPos, float2 MaxPos, float4 Color)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateLineElement(Ctx, 4, Offset))
		{
			MinPos = float2(MinPos) / float2(Ctx.Config.Resolution);
			MaxPos = float2(MaxPos) / float2(Ctx.Config.Resolution);

			float3 Pos0 = float3(MinPos.x, MinPos.y, 0);
			float3 Pos1 = float3(MaxPos.x, MinPos.y, 0);
			float3 Pos2 = float3(MaxPos.x, MaxPos.y, 0);
			float3 Pos3 = float3(MinPos.x, MaxPos.y, 0);

			FLineElement Element;
			Element.bIsScreenSpace = true;
			Element.Color0 = Element.Color1 = Color;
			Element.Pos0 = Pos0; Element.Pos1 = Pos1; AddLineElement(Ctx, Element, Offset + 0);
			Element.Pos0 = Pos1; Element.Pos1 = Pos2; AddLineElement(Ctx, Element, Offset + 1);
			Element.Pos0 = Pos2; Element.Pos1 = Pos3; AddLineElement(Ctx, Element, Offset + 2);
			Element.Pos0 = Pos3; Element.Pos1 = Pos0; AddLineElement(Ctx, Element, Offset + 3);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// AABB

void AddAABBTWS(FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color)
{
	if (Ctx.bIsActive)
	{
		uint Offset = 0;
		if (AllocateLineElement(Ctx, 12, Offset))
		{
			float3 P0 = float3(Min.x, Min.y, Min.z);
			float3 P1 = float3(Max.x, Min.y, Min.z);
			float3 P2 = float3(Max.x, Max.y, Min.z);
			float3 P3 = float3(Min.x, Max.y, Min.z);
			float3 P4 = float3(Min.x, Min.y, Max.z);
			float3 P5 = float3(Max.x, Min.y, Max.z);
			float3 P6 = float3(Max.x, Max.y, Max.z);
			float3 P7 = float3(Min.x, Max.y, Max.z);

			FLineElement Element;
			Element.bIsScreenSpace = false;
			Element.Color0 = Element.Color1 = Color;

			Element.Pos0 = P0; Element.Pos1 = P1; AddLineElement(Ctx, Element, Offset + 0);
			Element.Pos0 = P1; Element.Pos1 = P2; AddLineElement(Ctx, Element, Offset + 1);
			Element.Pos0 = P2; Element.Pos1 = P3; AddLineElement(Ctx, Element, Offset + 2);
			Element.Pos0 = P3; Element.Pos1 = P0; AddLineElement(Ctx, Element, Offset + 3);

			Element.Pos0 = P4; Element.Pos1 = P5; AddLineElement(Ctx, Element, Offset + 4);
			Element.Pos0 = P5; Element.Pos1 = P6; AddLineElement(Ctx, Element, Offset + 5);
			Element.Pos0 = P6; Element.Pos1 = P7; AddLineElement(Ctx, Element, Offset + 6);
			Element.Pos0 = P7; Element.Pos1 = P4; AddLineElement(Ctx, Element, Offset + 7);

			Element.Pos0 = P0; Element.Pos1 = P4; AddLineElement(Ctx, Element, Offset + 8);
			Element.Pos0 = P1; Element.Pos1 = P5; AddLineElement(Ctx, Element, Offset + 9);
			Element.Pos0 = P2; Element.Pos1 = P6; AddLineElement(Ctx, Element, Offset +10);
			Element.Pos0 = P3; Element.Pos1 = P7; AddLineElement(Ctx, Element, Offset +11);
		}
	}
}

void AddAABBWS (FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color)	{ AddAABBTWS(Ctx, Min + Ctx.Config.TranslatedWorldOffset, Max + Ctx.Config.TranslatedWorldOffset, Color); }
void AddAABB   (FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color)	{ AddAABBTWS(Ctx, Min + Ctx.Config.TranslatedWorldOffset, Max + Ctx.Config.TranslatedWorldOffset, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Cross

void AddCrossTWS(FShaderPrintContext Ctx, float3 Pos, float Size, float4 Color)
{
	if (Ctx.bIsActive)
	{
		AddLineTWS(Ctx, Pos - float3(Size,0,0), Pos + float3(Size,0,0), Color, Color);
		AddLineTWS(Ctx, Pos - float3(0,Size,0), Pos + float3(0,Size,0), Color, Color);
		AddLineTWS(Ctx, Pos - float3(0,0,Size), Pos + float3(0,0,Size), Color, Color);
	}
}

void AddCrossWS (FShaderPrintContext Ctx, float3 Pos, float Size, float4 Color)	{ AddCrossTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, Size, Color); }
void AddCross   (FShaderPrintContext Ctx, float3 Pos, float Size, float4 Color)	{ AddCrossTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, Size, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Circles

void AddCircleSS(FShaderPrintContext Ctx, float2 Center, float Radius, float4 Color, uint SegmentCount = 16)
{
	if (Ctx.bIsActive)
	{
		const float AngleStep = 2.0f * PI * rcp(float(SegmentCount-1));
		float Angle = 0.0f;
		float2 PrevP = Center + float2(Radius, 0.0f);
		for (uint SegIt = 0; SegIt < SegmentCount; ++SegIt)
		{
			Angle += AngleStep;

			float S, C;
			sincos(Angle, S, C);
			float2 P = Center + float2(C, S) * Radius;

			AddLineSS(Ctx, PrevP, P, Color, Color);
			PrevP = P;
		}
	}
}

void AddCircleTWS(FShaderPrintContext Ctx, float3 Center, float3 Normal, float Radius, float4 Color, uint SegmentCount = 16)
{
	if (Ctx.bIsActive)
	{
		// Calculate two perpendicular unit vectors to the normal
		const float3 RefUp = abs(Normal.z) < 0.999f ? float3(0, 0, 1) : float3(1, 0, 0);
		float3 X = normalize(cross(RefUp, Normal));
		float3 Y = cross(Normal, X);
		X *= Radius;
		Y *= Radius;

		const float AngleStep = 2.0f * PI * rcp(float(SegmentCount-1));
		float Angle = 0.0f;
		float3 PrevP = Center + X;
		for (uint SegIt = 0; SegIt < SegmentCount; ++SegIt)
		{
			Angle += AngleStep;

			float S, C;
			sincos(Angle, S, C);
			float3 P = Center + X * C + Y * S;

			AddLineTWS(Ctx, PrevP, P, Color, Color);
			PrevP = P;
		}
	}
}

void AddCircleWS (FShaderPrintContext Ctx, float3 Center, float3 Normal, float Radius, float4 Color, uint SegmentCount = 16) { AddCircleTWS(Ctx, Center + Ctx.Config.TranslatedWorldOffset, Normal, Radius, Color, SegmentCount); }
void AddCircle   (FShaderPrintContext Ctx, float3 Center, float3 Normal, float Radius, float4 Color, uint SegmentCount = 16) { AddCircleTWS(Ctx, Center + Ctx.Config.TranslatedWorldOffset, Normal, Radius, Color, SegmentCount); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Spheres

void AddSphereTWS(FShaderPrintContext Ctx, float3 Center, float Radius, float4 Color, uint SegmentCount = 16)
{
	AddCircleTWS(Ctx, Center, float3(1, 0, 0), Radius, Color, SegmentCount);
	AddCircleTWS(Ctx, Center, float3(0, 1, 0), Radius, Color, SegmentCount);
	AddCircleTWS(Ctx, Center, float3(0, 0, 1), Radius, Color, SegmentCount);
}

void AddSphereWS (FShaderPrintContext Ctx, float3 Center, float Radius, float4 Color, uint SegmentCount = 16) { AddSphereTWS(Ctx, Center + Ctx.Config.TranslatedWorldOffset, Radius, Color, SegmentCount); }
void AddSphere   (FShaderPrintContext Ctx, float3 Center, float Radius, float4 Color, uint SegmentCount = 16) { AddSphereTWS(Ctx, Center + Ctx.Config.TranslatedWorldOffset, Radius, Color, SegmentCount); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Referential

void AddReferentialTWS(FShaderPrintContext Ctx, float3 Pos, float3 T, float3 B, float3 N, float Scale = 1)
{
	if (Ctx.bIsActive)
	{
		AddLineTWS(Ctx, Pos, Pos + normalize(T)*Scale, ColorRed, ColorRed);
		AddLineTWS(Ctx, Pos, Pos + normalize(B)*Scale, ColorGreen, ColorGreen);
		AddLineTWS(Ctx, Pos, Pos + normalize(N)*Scale, ColorBlue, ColorBlue);
	}
}

void AddReferentialTWS(FShaderPrintContext Ctx, float3 Pos, float3x3 InM, float Scale = 1)
{
	AddReferentialTWS(Ctx, Pos, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale);
}

void AddReferentialTWS(FShaderPrintContext Ctx, float4x4 InM, float Scale = 1)
{
	AddReferentialTWS(Ctx, InM[3].xyz, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale);
}

void AddReferentialTWS(FShaderPrintContext Ctx, float3 Pos, float3 TangentZ, float Scale = 1)
{
	if (Ctx.bIsActive)
	{
		const float Sign = TangentZ.z >= 0 ? 1 : -1;
		const float a = -rcp(Sign + TangentZ.z);
		const float b = TangentZ.x * TangentZ.y * a;

		const float3 TangentX = { 1 + Sign * a * Pow2(TangentZ.x), Sign * b, -Sign * TangentZ.x };
		const float3 TangentY = { b,  Sign + a * Pow2(TangentZ.y), -TangentZ.y };

		AddReferentialTWS(Ctx, Pos, TangentX, TangentY, TangentZ, Scale);
	}
}

void AddReferentialWS (FShaderPrintContext Ctx, float4x4 InM, float Scale = 1)								{ AddReferentialTWS(Ctx, InM[3].xyz + Ctx.Config.TranslatedWorldOffset, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale); } 
void AddReferentialWS (FShaderPrintContext Ctx, float3 Pos, float3x3 InM, float Scale = 1)					{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale); } 
void AddReferentialWS (FShaderPrintContext Ctx, float3 Pos, float3 TangentZ, float Scale = 1)				{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, TangentZ, Scale); } 
void AddReferentialWS (FShaderPrintContext Ctx, float3 Pos, float3 T, float3 B, float3 N, float Scale = 1)	{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, T, B, N, Scale); } 
void AddReferential   (FShaderPrintContext Ctx, float4x4 InM, float Scale = 1)								{ AddReferentialTWS(Ctx, InM[3].xyz + Ctx.Config.TranslatedWorldOffset, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale); } 
void AddReferential   (FShaderPrintContext Ctx, float3 Pos, float3x3 InM, float Scale = 1)					{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, InM[0].xyz, InM[1].xyz, InM[2].xyz, Scale); } 
void AddReferential   (FShaderPrintContext Ctx, float3 Pos, float3 TangentZ, float Scale = 1)				{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, TangentZ, Scale); } 
void AddReferential   (FShaderPrintContext Ctx, float3 Pos, float3 T, float3 B, float3 N, float Scale = 1)	{ AddReferentialTWS(Ctx, Pos + Ctx.Config.TranslatedWorldOffset, T, B, N, Scale); } 

///////////////////////////////////////////////////////////////////////////////////////////////////
// Triangle

void AddLineTriangleTWS(FShaderPrintContext Ctx, float3 P0, float3 P1, float3 P2, float4 Color)
{
	if (Ctx.bIsActive)
	{
		AddLineTWS(Ctx, P0, P1, Color, Color);
		AddLineTWS(Ctx, P1, P2, Color, Color);
		AddLineTWS(Ctx, P2, P0, Color, Color);
	}
}

void AddLineTriangleWS (FShaderPrintContext Ctx, float3 P0, float3 P1, float3 P2, float4 Color)	{ AddLineTriangleTWS(Ctx, P0 + Ctx.Config.TranslatedWorldOffset, P1 + Ctx.Config.TranslatedWorldOffset, P2 + Ctx.Config.TranslatedWorldOffset, Color); }
void AddLineTriangle   (FShaderPrintContext Ctx, float3 P0, float3 P1, float3 P2, float4 Color)	{ AddLineTriangleTWS(Ctx, P0 + Ctx.Config.TranslatedWorldOffset, P1 + Ctx.Config.TranslatedWorldOffset, P2 + Ctx.Config.TranslatedWorldOffset, Color); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// OBB

void AddOBBTWS(FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color, float4x4 Transform, bool bHomogeneousDivide = false)
{
	if (Ctx.bIsActive) 
	{
		uint Offset = 0;
		if (AllocateLineElement(Ctx, 12, Offset))
		{
			float4 P0 = mul(float4(Min.x, Min.y, Min.z, 1.0f), Transform);
			float4 P1 = mul(float4(Max.x, Min.y, Min.z, 1.0f), Transform);
			float4 P2 = mul(float4(Max.x, Max.y, Min.z, 1.0f), Transform);
			float4 P3 = mul(float4(Min.x, Max.y, Min.z, 1.0f), Transform);
			float4 P4 = mul(float4(Min.x, Min.y, Max.z, 1.0f), Transform);
			float4 P5 = mul(float4(Max.x, Min.y, Max.z, 1.0f), Transform);
			float4 P6 = mul(float4(Max.x, Max.y, Max.z, 1.0f), Transform);
			float4 P7 = mul(float4(Min.x, Max.y, Max.z, 1.0f), Transform);

			if (bHomogeneousDivide)
			{
				P0.xyz /= P0.w;
				P1.xyz /= P1.w;
				P2.xyz /= P2.w;
				P3.xyz /= P3.w;
				P4.xyz /= P4.w;
				P5.xyz /= P5.w;
				P6.xyz /= P6.w;
				P7.xyz /= P7.w;
			}
			FLineElement Element;
			Element.bIsScreenSpace = false;
			Element.Color0 = Element.Color1 = Color;

			Element.Pos0 = P0.xyz; Element.Pos1 = P1.xyz; AddLineElement(Ctx, Element, Offset + 0);
			Element.Pos0 = P1.xyz; Element.Pos1 = P2.xyz; AddLineElement(Ctx, Element, Offset + 1);
			Element.Pos0 = P2.xyz; Element.Pos1 = P3.xyz; AddLineElement(Ctx, Element, Offset + 2);
			Element.Pos0 = P3.xyz; Element.Pos1 = P0.xyz; AddLineElement(Ctx, Element, Offset + 3);

			Element.Pos0 = P4.xyz; Element.Pos1 = P5.xyz; AddLineElement(Ctx, Element, Offset + 4);
			Element.Pos0 = P5.xyz; Element.Pos1 = P6.xyz; AddLineElement(Ctx, Element, Offset + 5);
			Element.Pos0 = P6.xyz; Element.Pos1 = P7.xyz; AddLineElement(Ctx, Element, Offset + 6);
			Element.Pos0 = P7.xyz; Element.Pos1 = P4.xyz; AddLineElement(Ctx, Element, Offset + 7);

			Element.Pos0 = P0.xyz; Element.Pos1 = P4.xyz; AddLineElement(Ctx, Element, Offset + 8);
			Element.Pos0 = P1.xyz; Element.Pos1 = P5.xyz; AddLineElement(Ctx, Element, Offset + 9);
			Element.Pos0 = P2.xyz; Element.Pos1 = P6.xyz; AddLineElement(Ctx, Element, Offset + 10);
			Element.Pos0 = P3.xyz; Element.Pos1 = P7.xyz; AddLineElement(Ctx, Element, Offset + 11);
		}
	}
}

void AddOBBWS (FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color, float4x4 Transform, bool bHomogeneousDivide = false)	{ Transform[3].xyz += Ctx.Config.TranslatedWorldOffset; AddOBBTWS(Ctx, Min, Max, Color, Transform, bHomogeneousDivide); }
void AddOBB   (FShaderPrintContext Ctx, float3 Min, float3 Max, float4 Color, float4x4 Transform, bool bHomogeneousDivide = false)	{ Transform[3].xyz += Ctx.Config.TranslatedWorldOffset; AddOBBTWS(Ctx, Min, Max, Color, Transform, bHomogeneousDivide); }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Strings
struct FStringInfo
{
	uint EntryID;
	uint Length;
	uint Offset;
	uint Pad;
};

FStringInfo UnpackStringInfo(uint2 In)
{
	FStringInfo Out = (FStringInfo)0;
	Out.EntryID = In.x;
	Out.Offset  = (In.y) & 0xFFFF;
	Out.Length  = (In.y >> 16) & 0xFF;
	return Out;
}

FStringInfo FindStringInfo(uint InEntryID, uint InInfoCount, StructuredBuffer<uint2> InInfoBuffer)
{
	FStringInfo Out = (FStringInfo)0;
	Out.EntryID = ~0;
	for (uint It = 0; It < InInfoCount; ++It)
	{
		if (InInfoBuffer[It].x == InEntryID)
		{
			return UnpackStringInfo(InInfoBuffer[It]);
		}
	}
	return Out;
}

void PrintString(inout FShaderPrintContext Ctx, uint InEntryID, FFontColor InColor, uint InInfoCount, uint InCharCount, StructuredBuffer<uint2> InInfoBuffer, Buffer<uint> InCharBuffer)
{
	const FStringInfo Info = FindStringInfo(InEntryID, InInfoCount, InInfoBuffer);
	if (Info.EntryID != ~0 && (Info.Length + Info.Offset) <= InCharCount)
	{
		for (uint It = 0; It < Info.Length; ++It)
		{
			const uint Char = InCharBuffer[It + Info.Offset];
			PrintSymbol(Ctx, Char, InColor);
		}
	}
}

uint LenString(uint InEntryID, uint InInfoCount, StructuredBuffer<uint2> InInfoBuffer)
{
	const FStringInfo Info = FindStringInfo(InEntryID, InInfoCount, InInfoBuffer);
	return Info.EntryID != ~0 ? Info.Length : 0;
}

#define FSTRINGS(N) \
	uint					N##_InfoCount;\
	uint					N##_CharCount;\
	StructuredBuffer<uint2> N##_InfoBuffer;\
	Buffer<uint>			N##_CharBuffer;\
	void Print##N(inout FShaderPrintContext Ctx, uint InEntryID, FFontColor InColor)\
	{\
		PrintString(Ctx, InEntryID, InColor, N##_InfoCount, N##_CharCount, N##_InfoBuffer, N##_CharBuffer);\
	}\
	uint Len##N(uint InEntryID)\
	{\
		return LenString(InEntryID, N##_InfoCount, N##_InfoBuffer);\
	}\
	uint Num##N()\
	{\
		return N##_InfoCount;\
	}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Current pixel

// Store the current pixel coordinates, which allow to retrieve it in any shaders once set.
// This data is only visible for the current thread/pixel, not available for neighbors
static uint2 ShaderPrintCurrentPixelCoord;

// Set current pixel coordinates
void SetShaderPrintCurrentPixelCoord(uint2 In)
{
	ShaderPrintCurrentPixelCoord = In;
}

// Get current pixel coordinates
uint2 GetShaderPrintCurrentPixelCoord()
{
	return ShaderPrintCurrentPixelCoord;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helper functions

// Print a value onto a single line
#define PrintLineN(Ctx, Value) Print(Ctx, TEXT(#Value), FontWhite); Print(Ctx, TEXT(" : "), FontWhite); Print(Ctx, Value, FontYellow); Newline(Ctx);
#define PrintLine(Ctx, Value) Print(Ctx, TEXT(#Value), FontWhite); Print(Ctx, TEXT(" : "), FontWhite); Print(Ctx, Value, FontYellow);

// Draw background based on the printed text rect
void AddTextBackground(inout FShaderPrintContext Ctx, float4 Color)
{
	if (Ctx.bIsActive)
	{
		AddFilledQuadSS(
			Ctx,
			(Ctx.RectMin - Ctx.Config.FontSize) * Ctx.Config.Resolution,
			(Ctx.RectMax + Ctx.Config.FontSize) * Ctx.Config.Resolution,
			Color);
	}
}