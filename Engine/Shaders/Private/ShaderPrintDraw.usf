// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShaderPrintDraw.usf:
	Shaders to gather contents of the debug value buffer, convert it to 
	symbols and draw each symbol as an instanced quad
=============================================================================*/

#include "Common.ush"	
#include "MiniFontCommon.ush"
#include "ShaderPrint.ush"

uint2 ViewportResolution;
uint FrameIndex;
uint FrameThreshold;

Buffer<uint> ValuesBuffer;
Buffer<uint> SymbolsBuffer;

RWBuffer<uint> RWValuesBuffer;
RWBuffer<uint> RWSymbolsBuffer;
RWBuffer<uint> RWStateBuffer;

RWBuffer<uint> RWIndirectDispatchArgsBuffer;
RWBuffer<uint> RWIndirectDrawArgsBuffer;


// --------------------------------------------------------------------------
// Clear the output value buffer
// Needs to be done at the beginning of each view
// Other shaders should only print debug values after this pass

[numthreads(1, 1, 1)]
void ClearCounterCS()
{
	// Symbol/print Counter
	ClearCounters(RWValuesBuffer);
}

// --------------------------------------------------------------------------
// Fill the indirect params buffer ready for the value->symbol compute pass
// Also clear the symbol buffer ready to be filled

[numthreads(1, 1, 1)]
void BuildIndirectDispatchArgsCS()
{
	uint ValueCount = min(ShaderPrintData.MaxCharacterCount, ValuesBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL]);
	WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, 0, (ValueCount + 63)/64, 1, 1);
	ClearCounters(RWSymbolsBuffer);
}

// --------------------------------------------------------------------------
// Read the values buffer and convert to the symbols buffer
// This is required for converting a single number to a full set of symbols
void AddSymbol(inout float2 ScreenPos, in int Symbol, in float3 Color, uint Type = SHADER_PRINT_TYPE_SYMBOL)
{
	FShaderPrintItem Item;
	Item.ScreenPos = ScreenPos;
	Item.Value = Symbol;
	Item.Type = Type;
	Item.Color = Color;
	Item.Metadata = 0;
	{
		// Buffer counter is stored in first element .Value
		int IndexToStore = 0;
		InterlockedAdd(RWSymbolsBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL], 1, IndexToStore);
	
		// Prevent writing off the buffer
		// Note that counter still increases so need to clamp when reading it in later passes
		if (uint(IndexToStore) >= ShaderPrintData.MaxSymbolCount)
		{
			return;
		}

		WriteSymbol(IndexToStore, Item, RWSymbolsBuffer);
	}

	ScreenPos.x += ShaderPrintData.FontSpacing.x;
}

void AddState(in uint InValue, in uint InPackedMetadata)
{
	FShaderPrintMetadata Metadata = ShaderPrintUnpackMetadata(InPackedMetadata);
	if (Metadata.Index == SHADER_PRINT_STATE_INVALID_INDEX)
	{
		// Change buffer to Ruint x 2 x cound
		InterlockedAdd(RWStateBuffer[SHADER_PRINT_STATE_COUNT_OFFSET], 1, Metadata.Index);
	}

	if (Metadata.Index < ShaderPrintData.MaxStateCount)
	{
		const uint Index3 = Metadata.Index * SHADER_PRINT_STATE_STRIDE + SHADER_PRINT_STATE_VALUE_OFFSET;
		RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_METADATA] = ShaderPrintPackMetadata(Metadata); // Re-encode with the new index
		RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_VALUE] = InValue;
		RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_FRAME] = FrameIndex;
	}
}

int GetDecimalSymbol(in int Digit)
{
	Digit = clamp(Digit, 0, 9);
	return (Digit + _0_);
}

int GetHexSymbol(in int Digit)
{
	Digit = clamp(Digit, 0, 15);
	return Digit < 10 ? Digit + _0_ : Digit - 10 + _A_;
}

void AddUIntSymbols(inout float2 ScreenPos, in uint Value, in float3 Color)
{
	const uint MaxSymbols = 10; // Symbols required for MAX_UINT
	uint SymbolCount = 1;
	uint Divisor = 1;
	uint TestValue = Value;
	{
		for (uint i = 0; i < MaxSymbols-1; ++i)
		{
			TestValue /= 10;
			if (TestValue > 0)
			{
				SymbolCount ++;
				Divisor *= 10;
			}
		}
	}
	
	{
		for (uint i = 0; i < SymbolCount; ++i)
		{
			uint Digit = Value / Divisor;
			AddSymbol(ScreenPos, GetDecimalSymbol((int)Digit), Color);
			Value = Value - (Digit * Divisor);
			Divisor /= 10;
		}
	}
}

void AddIntSymbols(inout float2 ScreenPos, in int Value, in float3 Color)
{
	if (Value < 0)
	{
	   AddSymbol(ScreenPos, _MINUS_, Color);
	   Value = -Value;
	}

	AddUIntSymbols(ScreenPos, (uint)Value, Color);
}

void AddFloatSymbols(inout float2 ScreenPos, in float Value, in float3 Color, in uint MaxDecimalCount)
{
	if (isinf(Value))
	{
		AddSymbol(ScreenPos, _I_, Color);
		AddSymbol(ScreenPos, _N_, Color);
		AddSymbol(ScreenPos, _F_, Color);
		return;
	}

	if (isnan(Value))
	{
		AddSymbol(ScreenPos, _N_, Color);
		AddSymbol(ScreenPos, _A_, Color);
		AddSymbol(ScreenPos, _N_, Color);
		return;
	}

	if (Value < 0)
	{
	   AddSymbol(ScreenPos, _MINUS_, Color);
	   Value = -Value;
	}

	AddIntSymbols(ScreenPos, (int)Value, Color);
	AddSymbol(ScreenPos, _DOT_, Color);

	MaxDecimalCount = min(MaxDecimalCount, MAX_DECIMAL_COUNT);
	for (uint i = MaxDecimalCount; i > 0; --i)
	{
		Value = frac(Value);
		if (Value > 0.f)
		{
			Value *= 10.f;
			AddSymbol(ScreenPos, GetDecimalSymbol((int)Value), Color);
		}
	}
}

void AddHexSymbols(inout float2 ScreenPos, uint Value, in float3 Color)
{
    AddSymbol(ScreenPos, _0_, Color);
    AddSymbol(ScreenPos, _X_, Color);
    
	for (int i = 7; i >= 0; --i)
    {
        uint Digit = (Value >> (i*4)) & 0xf;
        AddSymbol(ScreenPos, GetHexSymbol((int)Digit), Color);
    }
}

[numthreads(64, 1, 1)]
void BuildSymbolBufferCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint ValueCount = min(ShaderPrintData.MaxCharacterCount, ValuesBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL]);
	if (DispatchThreadId.x >= ValueCount)
	{
		return;
	}
	
	FShaderPrintItem Value = ReadSymbol(DispatchThreadId.x, ValuesBuffer);
	
	if (Value.Type == SHADER_PRINT_TYPE_SYMBOL)
		AddSymbol(Value.ScreenPos, Value.Value, Value.Color);
	else if (Value.Type == SHADER_PRINT_TYPE_FLOAT)
		AddFloatSymbols(Value.ScreenPos, asfloat(Value.Value), Value.Color, Value.Metadata);
	else if (Value.Type == SHADER_PRINT_TYPE_INT)
		AddIntSymbols(Value.ScreenPos, Value.Value, Value.Color);
	else if (Value.Type == SHADER_PRINT_TYPE_UINT)
		AddUIntSymbols(Value.ScreenPos, asuint(Value.Value), Value.Color);
	else if (Value.Type == SHADER_PRINT_TYPE_HEX)
		AddHexSymbols(Value.ScreenPos, asuint(Value.Value), Value.Color);
	else if (Value.Type == SHADER_PRINT_TYPE_SLIDER)
	{
		AddSymbol(Value.ScreenPos, Value.Value, Value.Color, SHADER_PRINT_TYPE_SLIDER);
		AddState(Value.Value, Value.Metadata);
	}
	else if (Value.Type == SHADER_PRINT_TYPE_CHECK)
	{
		AddSymbol(Value.ScreenPos, Value.Value, Value.Color, SHADER_PRINT_TYPE_CHECK);
		AddState(Value.Value, Value.Metadata);
	}
}

// --------------------------------------------------------------------------
// Compact state buffer

[numthreads(1, 1, 1)]
void CompactStateBufferCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint StateCount = min(RWStateBuffer[SHADER_PRINT_STATE_COUNT_OFFSET], ShaderPrintData.MaxStateCount);

	if (StateCount > 0)
	{
		for (uint Index = 0; Index < StateCount;)
		{
			const uint Index3 = Index * SHADER_PRINT_STATE_STRIDE + SHADER_PRINT_STATE_VALUE_OFFSET;
			const uint FrameDiff = FrameIndex - RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_FRAME];
			if (FrameDiff > FrameThreshold)
			{
				// Swap with last element
				const uint SrcIndex3 = (StateCount-1) * SHADER_PRINT_STATE_STRIDE + SHADER_PRINT_STATE_VALUE_OFFSET;
				RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_METADATA] = RWStateBuffer[SrcIndex3 + SHADER_PRINT_STATE_INDEX_METADATA];
				RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_VALUE]    = RWStateBuffer[SrcIndex3 + SHADER_PRINT_STATE_INDEX_VALUE];
				RWStateBuffer[Index3 + SHADER_PRINT_STATE_INDEX_FRAME]    = RWStateBuffer[SrcIndex3 + SHADER_PRINT_STATE_INDEX_FRAME];	
				--StateCount;
			}
			else
			{
				++Index;
			}
		}
		RWStateBuffer[0] = StateCount;
	}
}

// --------------------------------------------------------------------------
// Fill the indirect params buffer ready for the instanced draw pass

[numthreads(1, 1, 1)]
void BuildIndirectDrawArgsCS()
{
	const uint SymbolCount = min(ShaderPrintData.MaxSymbolCount, SymbolsBuffer[SHADER_PRINT_COUNTER_OFFSET_SYMBOL]);
	const uint IndexCount = 6;
	const uint InstanceCount = SymbolCount * 2; // Main glyph + drop shadow;
	WriteIndirectIndexedDrawArgs(RWIndirectDrawArgsBuffer, 0, IndexCount, InstanceCount);
}


// --------------------------------------------------------------------------
// Instanced draw call to render each symbol

void DrawSymbolsVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 OutPosition : SV_POSITION,
	out float2 OutUV : TEXCOORD0,
	out int OutType : TEXCOORD1,
	out int OutValue : TEXCOORD2,
	out int OutShadow : TEXCOORD3,
	out float3 OutColor : TEXCOORD4)
{
	bool bDropShadow = (InstanceId & 1) == 0;
	OutShadow = bDropShadow ? 1 : 0;
	float2 DropShadowOffset = bDropShadow ? ShaderPrintData.FontSize.xy * float2(0.2f, -0.2f) : 0;

	InstanceId = InstanceId >> 1;
	FShaderPrintItem Symbol = ReadSymbol(InstanceId, SymbolsBuffer);
	OutType = Symbol.Type;
	OutValue = Symbol.Value;
	OutColor = Symbol.Color;
	OutUV = float2(((VertexId + 1) / 3) & 1, VertexId & 1);
	float2 VertPos = float2(OutUV.x, 1.f - OutUV.y) * 2.f - 1.f;
	float2 InstancePos = float2(Symbol.ScreenPos.x, 1.f - Symbol.ScreenPos.y) * 2.f - 1.f;

	bool bIsValid = true;
	if (Symbol.Type == SHADER_PRINT_TYPE_SLIDER)
	{
		const float2 NormalizedSize = GetSliderWidgetSize(ShaderPrintData.FontSpacing, ShaderPrintData.Resolution) / float2(ShaderPrintData.Resolution);
		const float2 Offset = float2(NormalizedSize.x - NormalizedSize.y/4.f, 0.f);
		OutPosition.xy = VertPos.xy * NormalizedSize + Offset + InstancePos + DropShadowOffset;
		bIsValid = !bDropShadow;
	}
	else if (Symbol.Type == SHADER_PRINT_TYPE_CHECK)
	{
		const float2 NormalizedSize = GetCheckboxWidgetSize(ShaderPrintData.FontSpacing, ShaderPrintData.Resolution) / float2(ShaderPrintData.Resolution);
		const float2 Offset = float2(NormalizedSize.x * 0.5f, 0.f);
		OutPosition.xy = VertPos.xy * NormalizedSize + Offset + InstancePos + DropShadowOffset;
		bIsValid = !bDropShadow;
	}
	else
	{	
		OutPosition.xy = VertPos.xy * ShaderPrintData.FontSize.xy + InstancePos + DropShadowOffset;
	}

	OutPosition.zw = float2(0, 1);

	if (!bIsValid)
	{
		OutPosition = float4(INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT, INFINITE_FLOAT);
	}
}

bool IsInside(float2 InUV, float2 InDistToBorder)
{
	return 
		InUV.x > InDistToBorder.x && InUV.x < (1.f - InDistToBorder.x) &&
		InUV.y > InDistToBorder.y && InUV.y < (1.f - InDistToBorder.y);
}

bool IsOutside(float2 InUV, float2 InDistToBorder)
{
	return 
		InUV.x < InDistToBorder.x || InUV.x > (1.f - InDistToBorder.x) ||
		InUV.y < InDistToBorder.y || InUV.y > (1.f - InDistToBorder.y);
}

void DrawSymbolsPS(
	in float4 InPosition : SV_POSITION,
	in noperspective float2 InUV : TEXCOORD0,
	in nointerpolation int InType : TEXCOORD1,
	in nointerpolation int InValue : TEXCOORD2,
	in nointerpolation int InShadow : TEXCOORD3,
	in nointerpolation float3 InColor : TEXCOORD4,
	out float4 OutColor : SV_Target0)
{
	float Alpha = 1.0f;
	if (InType == SHADER_PRINT_TYPE_SLIDER)
	{
		const float2 PixelSizeInUV = 1.f / GetSliderWidgetSize(ShaderPrintData.FontSpacing, ShaderPrintData.Resolution).x;
		const float2 OuterBorder = float2(PixelSizeInUV.x, PixelSizeInUV.y * 10);
		const float2 InnerBorder = OuterBorder * 2;

		Alpha = 0.f;
		if (IsInside(InUV, InnerBorder))
		{
			Alpha = InUV.x < asfloat(InValue) ? 1.f : 0.25f;
		}
		if (IsOutside(InUV, OuterBorder))
		{
			Alpha = 1.f;
		}

	}
	else if (InType == SHADER_PRINT_TYPE_CHECK)
	{
		const float2 PixelSizeInUV = 1.f / GetCheckboxWidgetSize(ShaderPrintData.FontSpacing, ShaderPrintData.Resolution).x;
		const float2 OuterBorder = PixelSizeInUV;
		const float2 InnerBorder = OuterBorder * 2;

		Alpha = 0.f;
		if (IsInside(InUV, InnerBorder))
		{
			Alpha = (asuint(InValue) & 0x1) > 0 ? 1.f : 0.f;
		}
		if (IsOutside(InUV, OuterBorder))
		{
			Alpha = 1.f;
		}
	}
	else
	{
		Alpha = SampleMiniFont(InValue, (uint2)floor(InUV * 8.f));
	}	

	float3 Color = InColor * (InShadow == 0 ? Alpha : 0);
	OutColor = float4(Color, Alpha);
}


// --------------------------------------------------------------------------
// Zoomed output
#if SHADER_ZOOM
Texture2D<float4> InTexture;
RWTexture2D<float4> OutTexture;

uint PixelExtent;
uint ZoomFactor;
uint Corner;
int2 Resolution;

bool IsValid(int2 InCoord)
{
	return all(InCoord >= 0) && all(InCoord < Resolution);
}

bool IsOnBorder(int2 In, int2 MinB, int2 MaxB)
{
	return all(In <= MaxB) && all(In >= MinB) && 
		(In.x == MinB.x || In.x == MaxB.x ||
		 In.y == MinB.y || In.y == MaxB.y);
}

[numthreads(8, 8, 1)]
void DrawZoomCS(int2 DispatchThreadId : SV_DispatchThreadID)
{	
	if (any(DispatchThreadId > PixelExtent * 2))
	{
		return;
	}

	const int2 InCoord = DispatchThreadId + ShaderPrintData.CursorCoord - PixelExtent.xx;
	const int2 Offset = int2(10, 10);
	const int2 OutMaxPixel = (PixelExtent * 2 + 1) * ZoomFactor;

	int2 OutOffset = Offset;
	switch(Corner)
	{
	case 0: OutOffset = Offset; break;
	case 1: OutOffset = int2(Resolution.x - Offset.x - OutMaxPixel.x, Offset.y); break;
	case 2: OutOffset = Resolution - Offset - OutMaxPixel; break;
	case 3: OutOffset = int2(Offset.x, Resolution.y - Offset.y - OutMaxPixel.y); break;
	};

	const int2 CenterCoord = OutOffset + PixelExtent * ZoomFactor;
	const int2 MinCenter = CenterCoord;
	const int2 MaxCenter = CenterCoord + 1 * ZoomFactor;

	if (IsValid(ShaderPrintData.CursorCoord) && IsValid(InCoord))
	{
		const float4 Color = InTexture.Load(uint3(InCoord, 0));
		for (uint Y=0; Y<ZoomFactor; ++Y)
		for (uint X=0; X<ZoomFactor; ++X)
		{
			const int2 OutCoord = DispatchThreadId * ZoomFactor + int2(X, Y) + OutOffset;

			// External border
			const bool bIsBorder0 = any(OutCoord == OutOffset) || any(OutCoord == OutOffset + OutMaxPixel - 1);
			// Pixel center border
			const bool bIsBorder1 = IsOnBorder(OutCoord, MinCenter, MaxCenter);

			if (IsValid(OutCoord))
			{
				OutTexture[OutCoord] = bIsBorder0 || bIsBorder1 ? float4(1,1,0,1) : Color;
			}

		}
	}
}
#endif // SHADER_ZOOM

// --------------------------------------------------------------------------
// Upload parameters
#if SHADER_UPLOAD

// Copy ShaderPrintData uniform parameter within UEDiagnosticBuffer
[numthreads(1, 1, 1)]
void UploadCS(int2 DispatchThreadId : SV_DispatchThreadID)
{	
#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	FShaderPrintConfig In;
	In.Resolution = ShaderPrintData.Resolution;
	In.CursorCoord = ShaderPrintData.CursorCoord;
	In.TranslatedWorldOffset = ShaderPrintData.TranslatedWorldOffset;
	In.FontSize = ShaderPrintData.FontSize;
	In.FontSpacing = ShaderPrintData.FontSpacing;
	In.MaxCharacterCount = ShaderPrintData.MaxCharacterCount;
	In.MaxSymbolCount = ShaderPrintData.MaxSymbolCount;
	In.MaxStateCount = ShaderPrintData.MaxStateCount;
	In.MaxLineCount = ShaderPrintData.MaxLineCount;
	In.MaxTriangleCount = ShaderPrintData.MaxTriangleCount;
	In.bIsDrawLocked = ShaderPrintData.IsDrawLocked;
	PackShaderPrintConfig(In);

	// Clear counters
	FShaderPrintContext Ctx; // Dummy context
	SHADER_PRINT_RWENTRYBUFFER(Ctx, SHADER_PRINT_COUNTER_OFFSET_LINE) = 0;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, SHADER_PRINT_COUNTER_OFFSET_TRIANGLE) = 0;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, SHADER_PRINT_COUNTER_OFFSET_SYMBOL) = 0;
	SHADER_PRINT_RWENTRYBUFFER(Ctx, SHADER_PRINT_COUNTER_OFFSET_FREE) = 0;

#endif
}
#endif //SHADER_UPLOAD

// --------------------------------------------------------------------------
// Copy data prior to draw
#if SHADER_COPY

#define TYPE_LINE 0
#define TYPE_TRIANGLE 1
#define TYPE_SYMBOL 2

#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
void CopyDataType(FShaderPrintConfig Config, uint Type)
{
	uint CounterOffset = 0;
	uint DataTypeSizeInUint = 0;
	uint DataOffset0 = 0;

	if (Type == TYPE_SYMBOL)
	{
		CounterOffset = SHADER_PRINT_COUNTER_OFFSET_SYMBOL;
		DataTypeSizeInUint = SHADER_PRINT_UINT_COUNT_SYMBOL;
		DataOffset0 = GetSymbolOffset(0);
	}
	else if (Type == TYPE_LINE)
	{
		CounterOffset = SHADER_PRINT_COUNTER_OFFSET_LINE; 		
		DataTypeSizeInUint = SHADER_PRINT_UINT_COUNT_LINE;
		DataOffset0 = GetPrimitiveLineOffset(0, Config.MaxCharacterCount);
	}
	else if (Type == TYPE_TRIANGLE)
	{
		CounterOffset = SHADER_PRINT_COUNTER_OFFSET_TRIANGLE;	
		DataTypeSizeInUint = SHADER_PRINT_UINT_COUNT_TRIANGLE;
		DataOffset0 = GetPrimitiveTriangleOffset(0, Config.MaxCharacterCount, Config.MaxLineCount);	
	}
	else
	{
		return;
	}

	FShaderPrintContext Ctx; // Dummy context
	const uint SrcCount = SHADER_PRINT_RWENTRYBUFFER(Ctx, CounterOffset);
	const uint DstCount = RWValuesBuffer[CounterOffset];

	// Compute the offset of the first item:
	// * For the src buffer (i.e. diagonistoc buffer), it is the offset to the first element
	// * For the dst buffer (i.e. shader print rdg buffer), it is the offset to the last element 
	//   which was happens through explicit binding
	const uint SrcOffsetInUint = DataOffset0 + SHADER_PRINT_DIAGNOSTIC_BUFFER_ENTRY_OFFSET;
	const uint DstOffsetInUint = DataOffset0 + DstCount * DataTypeSizeInUint;

	// Copy data + counter
	// TODO: make it parallel
	const uint CountInUint = SrcCount * DataTypeSizeInUint;
	for (uint It = 0; It < CountInUint; ++It)
	{
		const uint SrcIndex = SrcOffsetInUint + It;
		const uint DstIndex = DstOffsetInUint + It;
		RWValuesBuffer[DstIndex] = UEDiagnosticBuffer[SrcIndex];
	}
	RWValuesBuffer[CounterOffset] = SrcCount + DstCount;
}
#endif

// Copy/merge data from UEDiagnosticBuffer into ShaderPrint RDG buffer for display
[numthreads(1, 1, 1)]
void CopyCS(int2 DispatchThreadId : SV_DispatchThreadID)
{	
#if SHADER_PRINT_USE_DIAGNOSTIC_BUFFER
	if (any(DispatchThreadId != 0))
	{
		return;
	}

	FShaderPrintConfig Config = UnpackShaderPrintConfig();

	CopyDataType(Config, TYPE_LINE);
	CopyDataType(Config, TYPE_TRIANGLE);
	CopyDataType(Config, TYPE_SYMBOL);
#endif
}
#endif // SHADER_COPY
