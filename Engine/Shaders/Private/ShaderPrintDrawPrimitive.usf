// Copyright Epic Games, Inc. All Rights Reserved.


#include "Common.ush"
#include "ShaderPrintCommon.ush"

float3 TranslatedWorldOffsetConversion;
uint bCheckerboardEnabled;
uint bDrawOccludedLines;

// The shader used to render the debug element on view
Buffer<uint> ShaderDrawDebugPrimitive;

Texture2D<float> DepthTexture;
float2 OutputInvResolution; 
float2 OriginalViewRectMin;
float2 OriginalViewSize;
float2 OriginalBufferInvSize;
float4x4 TranslatedWorldToClip;

SamplerState DepthSampler;

#if GPU_DEBUG_RENDERING_LINE_VS 
void ShaderDrawDebugVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION,
	out float4 ColorAlpha : TEXCOORD0)
{
	const FLineElement Item = UnpackLineElement(ShaderDrawDebugPrimitive, InstanceId, ShaderPrintData.MaxCharacterCount);

	float4 VertexPositionTWS = float4(VertexId == 0 ? Item.Pos0 : Item.Pos1, 1.0f);
	float4 VertexColor = VertexId == 0 ? Item.Color0 : Item.Color1;

	if (Item.bIsScreenSpace)
	{
		Position = VertexPositionTWS;
		Position.xy = Position.xy * 2 - 1;
		Position.y = -Position.y;

		Position.z = 1;
		Position.w = 1;
	}
	else
	{
		// Optional world offset. For 'locked' debug drawing, this offset translates the world 
		// position recorded for a given translated world offset, into the current translated 
		// world offset
		VertexPositionTWS.xyz += TranslatedWorldOffsetConversion;

		Position = mul(VertexPositionTWS, TranslatedWorldToClip);
	}
	ColorAlpha = VertexColor;
}
#endif

#if GPU_DEBUG_RENDERING_TRIANGLE_VS 
void ShaderDrawDebugVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION,
	out float4 ColorAlpha : TEXCOORD0)
{
	const FTriangleElement Item = UnpackTriangleElement(ShaderDrawDebugPrimitive, InstanceId, ShaderPrintData.MaxCharacterCount, ShaderPrintData.MaxLineCount);

	float4 VertexPositionTWS = float4(VertexId == 0 ? Item.Pos0 : (VertexId == 1 ? Item.Pos1 : Item.Pos2), 1.0f);
	float4 VertexColor = Item.Color;

	if (Item.bIsScreenSpace)
	{
		Position = VertexPositionTWS;
		Position.xy = Position.xy * 2 - 1;
		Position.y = -Position.y;

		Position.z = 1;
		Position.w = 1;
	}
	else
	{
		// Optional world offset. For 'locked' debug drawing, this offset translates the world 
		// position recorded for a given translated world offset, into the current translated 
		// world offset
		VertexPositionTWS.xyz += TranslatedWorldOffsetConversion;

		Position = mul(VertexPositionTWS, TranslatedWorldToClip);
	}
	ColorAlpha = VertexColor;
}
#endif

#if GPU_DEBUG_RENDERING_PS
void ShaderDrawDebugPS(
	in float4 Position : SV_POSITION,
	in float4 ColorAlpha : TEXCOORD0,
	out float4 OutSceneColor : SV_Target0)
{
	uint2 PixelCoord = Position.xy;

	const float2 ViewportUV = float2(PixelCoord) * OutputInvResolution;
	const float2 DepthUV    = (ViewportUV * OriginalViewSize.xy + OriginalViewRectMin) * OriginalBufferInvSize;
	const float Depth = DepthTexture.SampleLevel(DepthSampler, DepthUV, 0);
	const bool bIsHidden = Position.z < Depth; // Reverse-Z

	if (bIsHidden && !bDrawOccludedLines)
	{
		discard;
	}

	const float ColorScale = bIsHidden ? 0.4f : 1;
	const float CheckerboardScale = ((PixelCoord.x%2) ^ (PixelCoord.y%2)) > 0 ? 1 : 0.0;
	const float OcclusionScale = (bCheckerboardEnabled && bIsHidden) ? (bDrawOccludedLines ? CheckerboardScale : 0.0f) : 1;
	OutSceneColor = OcclusionScale * ColorAlpha * float4(ColorAlpha.aaa * ColorScale, 1.0); // Pre multiplied alpha with color scale
}
#endif

#if GPU_DEBUG_RENDERING_COPY_CS

Buffer<uint> ElementBuffer;
RWBuffer<uint> RWIndirectArgs;
uint PrimitiveType;

[numthreads(1, 1, 1)]
void ShaderDrawDebugCopyCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const bool bLine    = PrimitiveType == SHADER_PRINT_TYPE_LINE;
	const uint Index    = bLine ? SHADER_PRINT_COUNTER_OFFSET_LINE : SHADER_PRINT_COUNTER_OFFSET_TRIANGLE;
	const uint MaxCount = bLine ? ShaderPrintData.MaxLineCount : ShaderPrintData.MaxTriangleCount;

	const uint IndexCount = bLine ? 2 : 3; // Line or Triangle primitive
	const uint InstanceCount = min(MaxCount, ElementBuffer[Index]);
	WriteIndirectIndexedDrawArgs(RWIndirectArgs, 0, IndexCount, InstanceCount);
}
#endif