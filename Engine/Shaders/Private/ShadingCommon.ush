// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"

#ifndef SHADING_PATH_MOBILE
#define SHADING_PATH_MOBILE 0
#endif

#ifndef SHADING_PATH_DEFERRED
#define SHADING_PATH_DEFERRED 0
#endif

///////////////////////////////////////////////////////////////////////////////
// Shading models


// SHADINGMODELID_* occupy the 4 low bits of an 8bit channel and SKIP_* occupy the 4 high bits
#define SHADINGMODELID_UNLIT				0
#define SHADINGMODELID_DEFAULT_LIT			1
#define SHADINGMODELID_SUBSURFACE			2
#define SHADINGMODELID_PREINTEGRATED_SKIN	3
#define SHADINGMODELID_CLEAR_COAT			4
#define SHADINGMODELID_SUBSURFACE_PROFILE	5
#define SHADINGMODELID_TWOSIDED_FOLIAGE		6
#define SHADINGMODELID_HAIR					7
#define SHADINGMODELID_CLOTH				8
#define SHADINGMODELID_EYE					9
#define SHADINGMODELID_SINGLELAYERWATER		10
#define SHADINGMODELID_THIN_TRANSLUCENT		11
#define SHADINGMODELID_SUBSTRATE			12		// Temporary while we convert everything to Substrate
#define SHADINGMODELID_SUBSTRATE_TOON		13
#define SHADINGMODELID_NUM					14
#define SHADINGMODELID_MASK					0xF		// 4 bits reserved for ShadingModelID			

// The flags are defined so that 0 value has no effect!
// These occupy the 4 high bits in the same channel as the SHADINGMODELID_*
#define HAS_ANISOTROPY_MASK				(1 << 4)
#define SKIP_PRECSHADOW_MASK			(1 << 5)
#define ZERO_PRECSHADOW_MASK			(1 << 6)
#define IS_FIRST_PERSON_MASK			(1 << 6) // Technically not related to the SelectiveOutputMask, but IS_FIRST_PERSON_MASK shares a bit with ZERO_PRECSHADOW_MASK and consequently only works if ALLOW_STATIC_LIGHTING is 0.
#define SKIP_VELOCITY_MASK				(1 << 7)

#define SSS_PROFILE_ID_INVALID  255
#define SSS_PROFILE_ID_PERPIXEL 512

// This is selected so that the maximum of the LUT is <= 1.0. The maximum can be PI for D_GGX.
#define GGX_MAX_LUT_LOAD_SCALE		(PI)
#define GGX_MAX_LUT_STORE_SCALE		(1.0 / GGX_MAX_LUT_LOAD_SCALE)
#define GGX_MAX_LUT_RESOLUTION		32
#define GGX_MAX_LUT_LOAD_UV_SCALE	((float(GGX_MAX_LUT_RESOLUTION) - 1.0f) / float(GGX_MAX_LUT_RESOLUTION))
#define GGX_MAX_LUT_LOAD_UV_BIAS	(0.5f / float(GGX_MAX_LUT_RESOLUTION))

// for debugging and to visualize
float3 GetShadingModelColor(uint ShadingModelID)
{
	// TODO: PS4 doesn't optimize out correctly the switch(), so it thinks it needs all the Samplers even if they get compiled out
	//	This will get fixed after launch per Sony...
#if PS4_PROFILE
		 if (ShadingModelID == SHADINGMODELID_UNLIT) return float3(0.1f, 0.1f, 0.2f); // Dark Blue
	else if (ShadingModelID == SHADINGMODELID_DEFAULT_LIT) return float3(0.1f, 1.0f, 0.1f); // Green
	else if (ShadingModelID == SHADINGMODELID_SUBSURFACE) return float3(1.0f, 0.1f, 0.1f); // Red
	else if (ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN) return float3(0.6f, 0.4f, 0.1f); // Brown
	else if (ShadingModelID == SHADINGMODELID_CLEAR_COAT) return float3(0.1f, 0.4f, 0.4f); 
	else if (ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE) return float3(0.2f, 0.6f, 0.5f); // Cyan
	else if (ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE) return float3(0.2f, 0.2f, 0.8f); // Blue
	else if (ShadingModelID == SHADINGMODELID_HAIR) return float3(0.6f, 0.1f, 0.5f);
	else if (ShadingModelID == SHADINGMODELID_CLOTH) return float3(0.7f, 1.0f, 1.0f); 
	else if (ShadingModelID == SHADINGMODELID_EYE) return float3(0.3f, 1.0f, 1.0f); 
	else if (ShadingModelID == SHADINGMODELID_SINGLELAYERWATER) return float3(0.5f, 0.5f, 1.0f);
	else if (ShadingModelID == SHADINGMODELID_THIN_TRANSLUCENT) return float3(1.0f, 0.8f, 0.3f);
	else if (ShadingModelID == SHADINGMODELID_SUBSTRATE) return float3(1.0f, 1.0f, 0.0f);
	else return float3(1.0f, 1.0f, 1.0f); // White
#else
	switch(ShadingModelID)
	{
		case SHADINGMODELID_UNLIT: return float3(0.1f, 0.1f, 0.2f); // Dark Blue
		case SHADINGMODELID_DEFAULT_LIT: return float3(0.1f, 1.0f, 0.1f); // Green
		case SHADINGMODELID_SUBSURFACE: return float3(1.0f, 0.1f, 0.1f); // Red
		case SHADINGMODELID_PREINTEGRATED_SKIN: return float3(0.6f, 0.4f, 0.1f); // Brown
		case SHADINGMODELID_CLEAR_COAT: return float3(0.1f, 0.4f, 0.4f); // Brown
		case SHADINGMODELID_SUBSURFACE_PROFILE: return float3(0.2f, 0.6f, 0.5f); // Cyan
		case SHADINGMODELID_TWOSIDED_FOLIAGE: return float3(0.2f, 0.2f, 0.8f); // Cyan
		case SHADINGMODELID_HAIR: return float3(0.6f, 0.1f, 0.5f);
		case SHADINGMODELID_CLOTH: return float3(0.7f, 1.0f, 1.0f);
		case SHADINGMODELID_EYE: return float3(0.3f, 1.0f, 1.0f);
		case SHADINGMODELID_SINGLELAYERWATER: return float3(0.5f, 0.5f, 1.0f);
		case SHADINGMODELID_THIN_TRANSLUCENT: return float3(1.0f, 0.8f, 0.3f);
		case SHADINGMODELID_SUBSTRATE: return float3(1.0f, 1.0f, 0.0f);
		default: return float3(1.0f, 1.0f, 1.0f); // White
	}
#endif
}


#define SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING (MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || SUBSTRATE_ENABLED)

bool GetShadingModelRequiresBackfaceLighting(uint ShadingModelID)
{
	return ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE;
}

///////////////////////////////////////////////////////////////////////////////
// Shading parameterisation

float F0ToDielectricSpecular(float F0)
{
	return saturate(F0 / 0.08f);
}

float F0RGBToF0(float3 F0)
{
	return max3(F0.r, F0.g, F0.b);
}

float F0RGBToDielectricSpecular(float3 F0)
{
	return F0ToDielectricSpecular(F0RGBToF0(F0));
}

half DielectricSpecularToF0(half Specular)
{
	return half(0.08f * Specular);
}

// [Burley, "Extending the Disney BRDF to a BSDF with Integrated Subsurface Scattering"]
float DielectricF0ToIor(float F0)
{
	return 2.0f / (1.0f - sqrt(min(F0,0.99))) - 1.0f;
}

float DielectricF0RGBToIor(float3 F0)
{
	return DielectricF0ToIor(F0RGBToF0(F0));
}

float DielectricIorToF0(float Ior)
{
	const float F0Sqrt = (Ior-1)/(Ior+1);
	const float F0 = F0Sqrt*F0Sqrt;
	return F0;
}

// Note: n and k are usually measured in spectral domain
// There is no good/correct translation to linear sRGB
// One could use linear sRGB measurement curve's peaks
// * R primary ~ 610-630 nm
// * G primary ~ 530-550 nm
// * B primary ~ 450-470 nm
float3 ConductorIorToF0(float3 n, float3 k)
{
	const float3 k2 = k*k;
	const float3 F0 = (Pow2(n-1) + k2) / (Pow2(n+1) + k2);
	return F0;
}

// Anything with Specular less than 2% is physically impossible and is instead considered to be shadowing.
// See F_Schlick implementation.
float  GetF0MicroOcclusionThreshold() { return 0.02f; }
float  F0ToMicroOcclusion(float F0)   { return saturate(50.0 * F0); }
float3 F0ToMicroOcclusion(float3 F0)  { return saturate(50.0 * F0); }

float F0RGBToMicroOcclusion(float3 F0)
{
	return F0ToMicroOcclusion(F0RGBToF0(F0));
}

half3 ComputeF0(half Specular, half3 BaseColor, half Metallic)
{
	return lerp(DielectricSpecularToF0(Specular).xxx, BaseColor, Metallic.xxx);
}

float3 ComputeF90(float3 F0, float3 EdgeColor, float Metallic)
{
	return lerp(1.0, EdgeColor, Metallic.xxx);
}

float3 ComputeDiffuseAlbedo(float3 BaseColor, float Metallic)
{
	return BaseColor - BaseColor * Metallic;
}

float MakeRoughnessSafe(float Roughness, float MinRoughness=0.001f)
{
	return clamp(Roughness, MinRoughness, 1.0f);
}

float F0ToMetallic(float F0)
{
	// Approximate the metallic input from F0 with a small lerp region
	const float FullMetalBeginF0 = 0.08f;	// Instead of DiamondF0 = 0.24, the metallic region starts right after metallic >0 and specular=1 to match with legacy.
	const float FullMetalEndF0   = 0.4f;	// roughly the end of semi-conductor
	// This is compatible with UE shading model mapping allowing F0 to take a value up to 0.08 for dielectric.

	return saturate((F0 - FullMetalBeginF0) / (FullMetalEndF0 - FullMetalBeginF0));
}

float F0RGBToMetallic(float3 F0)
{
	return F0ToMetallic(F0RGBToF0(F0));
}


///////////////////////////////////////////////////////////////////////////////
// Layering, coverage and transmittance

struct FVerticalLayeringInfo
{
	float TransmittanceTopAndBottom;	// Ratio of transmittance of both top and bottom surface only
	float TransmittanceOnlyBottom;		// Ratio of transmittance of bottom surface only 
	float TransmittanceOnlyTop;			// Ratio of transmittance of top surface only 

	float SurfaceBottom;				// Ratio of bottom surface visible
	float SurfaceTop;					// Ratio of top surface visible

	float Coverage;						// Ratio of any surface is visible
	float NoSurface;					// Ratio of no surface is visible
};

// This function returns surface information after a layering of two slab of matter having difference and uncorelated coverage.
// See https://www.desmos.com/calculator/qg5yc3nnr4
FVerticalLayeringInfo GetVerticalLayeringInfo(const float TopCoverage, const float BottomCoverage)
{
	FVerticalLayeringInfo Info;

	Info.TransmittanceTopAndBottom	= TopCoverage * BottomCoverage;
	Info.TransmittanceOnlyBottom	= (1.0f - TopCoverage) * BottomCoverage;
	Info.TransmittanceOnlyTop		= (1.0f - BottomCoverage) * TopCoverage;

	Info.SurfaceBottom				= Info.TransmittanceOnlyBottom;
	Info.SurfaceTop					= TopCoverage;								// == Info.TransmittanceOnlyTop + Info.TransmittanceTopAndBottom;

	Info.Coverage					= Info.SurfaceTop + Info.SurfaceBottom;		// == Info.TransmittanceTopAndBottom + Info.TransmittanceOnlyBottom + Info.TransmittanceOnlyTop;// == TopCoverage + Info.TransmittanceOnlyBottom
	Info.NoSurface					= 1.0f - Info.Coverage;

	return Info;
}


