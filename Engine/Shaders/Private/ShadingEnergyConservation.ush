// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ShadingCommon.ush"

// If EnergyConservation flag is not enabled explicitly then:
// * If Substrate is enabled: ON by default
// * If Substrate is enabled with legacy blendable GBuffer: OFF by default for performance parity with legacy on low end platforms.
// * If Substrate is disabled: OFF by default
#ifdef USE_ENERGY_CONSERVATION
#error "USE_ENERGY_CONSERVATION is defined by this file, it should not be defined prior to including it"
#endif

#if PATH_TRACING && (SUBSTRATE_ENABLED || LEGACY_MATERIAL_ENERGYCONSERVATION)
	// When using the path tracer, we use energy conservation either when:
	//   * Substrate is enabled for the project
	//   * It is enabled in the project for legacy materials
	#define USE_ENERGY_CONSERVATION 1
#elif (SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT>0) || LEGACY_MATERIAL_ENERGYCONSERVATION
	#if SUPPORTS_INDEPENDENT_SAMPLERS 
		#define USE_ENERGY_CONSERVATION 1
	#else
		// Platforms that do not support shared sampler will in fact quickly reach the limit of samplers in material pixel shaders.
		// This is mostly true for OpenGLES. In this case, we enforce disabling energy conservation to be able to compile substrate materials.
		#define USE_ENERGY_CONSERVATION 0
	#endif
#else
	// SUBSTRATE_GBUFFER_FORMAT == 0 || LEGACY_MATERIAL_ENERGYCONSERVATION == 0
	#define USE_ENERGY_CONSERVATION 0
#endif

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedShadingEnergySampler			View.SharedBilinearClampedSampler
#else
	#define SharedShadingEnergySampler			View.ShadingEnergySampler
#endif

// Energy compensation/preservation for the various BxDFs. This file provides utility functions to tweak lobe weights
// 
// References:
// [1] "Revisiting physically based shading at Imageworks" - Christopher Kulla & Alejandro Conty
//   https://blog.selfshadow.com/publications/s2017-shading-course/imageworks/s2017_pbs_imageworks_slides_v2.pdf
// [2] "Practical multiple scattering compensation for microfacet models" - Emmanuel Turquin
//   https://blog.selfshadow.com/publications/turquin/ms_comp_final.pdf
// [3] "A Multiple-Scattering Microfacet Model for Real-Time Image Based Lighting" - Carmelo J. Fdez-Aguera
//   http://jcgt.org/published/0008/01/03/
// 
// We follow the terminology from [1]:
// Energy compensation: adjust individual BxDFs so that they have unit albedo from all viewing angles, compensating for (for example) the lack of multiple scattering in microfacet BxDFs
// Energy preservation: adjust the combination of lobes (such as diffuse+specular) to ensure we never create additional energy from any angle

// The suggested implementation in [1] is cumbersome for a GPU implementation because it requires table lookups during sampling to achieve low variance. We instead sacrifice reciprocity and
// simply divide the BxDF by its (tabulated/fit) directional albedo to enforce conservation of energy. We follow the approach outlined in [2] to account for spec/cloth and glass, and use
// the split-sum approach from [3] for the diffuse/specular energy preservation case (which generalizes to other lobes as well).

// USE_ENERGY_CONSERVATION controls the implementation details of this file
//    0: disable conservation, all bsdfs are assumed to already have unit albedo from all viewing angles
//    1: use a small baked texture to store the directional albedo (and fresnel weighted albedo)
//    2: use an analytic approximation to directional albedo (doesn't require any textures to be bound and can be faster -- WIP)

///////////////////////////////////////////////////////////////////////////////////////////////////
// LUT Lookups

float2 GGXEnergyLookupEnv(float Roughness, float NoV)
{
	return View.ShadingEnergyGGXSpecTexture.SampleLevel(SharedShadingEnergySampler, float2(NoV, Roughness), 0);
}

float2 GGXEnergyLookup(float Roughness, float NoV)
{
#if USE_ENERGY_CONSERVATION == 1
	return View.ShadingEnergyGGXSpecTexture.SampleLevel(SharedShadingEnergySampler, float2(NoV, Roughness), 0);
#elif USE_ENERGY_CONSERVATION == 2
	const float r = Roughness;
	const float c = NoV;
	const float E = 1.0 - saturate(pow(r, c / r) * ((r * c + 0.0266916) / (0.466495 + c)));
	const float Ef = Pow5(1 - c) * pow(2.36651 * pow(c, 4.7703 * r) + 0.0387332, r);
	return float2(E, Ef);
#else
	return float2(1, 0);
#endif
}

float GGXEnergyLookup(float Roughness, float NoV, float Eta)
{
	// Following [2] Eq 18, we simply divide the whole bsdf by its single scattering albedo to conserve energy
	// This break reciprocity, but is much simpler than the approach outlined in [1] which requires substantially more computation.
#if USE_ENERGY_CONSERVATION == 1
	// NOTE: Eta is encoded for [1,3] range only, energy loss will happens above 3.0 - but this keeps more resolution on the visually significant portion of the range
	float2 E = View.ShadingEnergyGGXGlassTexture.SampleLevel(SharedShadingEnergySampler, float3(NoV, Roughness, max(Eta, rcp(Eta)) * 0.5 - 0.5), 0);
	return Eta >= 1.0 ? E.x : E.y;
#else
	// TODO: find an analytic fit for this case
	return 1.0;
#endif
}

float GGXMaxEnergyLookup(float Roughness, float Anisotropy, float NoV)
{
	// Always evaluate since it is only used for toon shading as of today without any more options.
	const float4 ValuesAlongNoV = GGX_MAX_LUT_LOAD_SCALE * View.ShadingEnergyGGXMaxSpecTexture.SampleLevel(SharedShadingEnergySampler, float2(Roughness, abs(Anisotropy)) * GGX_MAX_LUT_LOAD_UV_SCALE + GGX_MAX_LUT_LOAD_UV_BIAS, 0).rgba;

	const float Step = 0.3333f;
#if 1
	const float4 StepScale = float4(0.0f, 0.3333f, 0.6666f, 1.0f);
	float4 LerpFactors = 1.0f - saturate(abs(NoV - StepScale) / Step);
	return dot(ValuesAlongNoV, LerpFactors);
#else
	if(NoV <= Step)
		return lerp(ValuesAlongNoV.x, ValuesAlongNoV.y, saturate((NoV - 0.0f * Step) / Step));
	else if(NoV <= 2.0f * Step)
		return lerp(ValuesAlongNoV.y, ValuesAlongNoV.z, saturate((NoV - 1.0f * Step) / Step));
	//else  if(NoV < 3.0f * Step)
		return lerp(ValuesAlongNoV.z, ValuesAlongNoV.w, saturate((NoV - 2.0f * Step) / Step));
#endif
}

float2 ClothEnergyLookup(float Roughness, float NoV)
{
#if USE_ENERGY_CONSERVATION == 1
	return View.ShadingEnergyClothSpecTexture.SampleLevel(SharedShadingEnergySampler, float2(NoV, Roughness), 0);
#elif USE_ENERGY_CONSERVATION == 2
	// Simple analytical fit (avoids building a table)
	// NOTE: this fit is not exact near grazing angles for roughness ~0.7
	const float c = NoV;
	const float r = Roughness;
	return float2(
		(0.526422 / ((-0.227114 + r) * (-0.968835 + r) * ((5.38869 - 20.2835 * c) * r) - (-1.18761 - ((2.58744 - c) * c)))) + 0.0615456,
		0.0 // TODO: is it worth capturing this? this term is close to 0 (and experiments with the tabulated form show it doesn't have much impact)
	);
#else
	return float2(1, 0);
#endif
}

float DiffuseEnergyLookup(float Roughness, float NoV)
{
#if USE_ENERGY_CONSERVATION == 1
	//return View.ShadingEnergyDiffuseTexture.SampleLevel(SharedShadingEnergySampler, float2(NoV, Roughness), 0);
	// For now we do not apply Chan diffuse energy preservation on diffuse ambiant.
	// This is because Chan is built for F=0.04 and unfortunately this causes ambient to darken a grazing angles.
	// SUBSTRATE_TODO Apply the inverse of Fresnel with F=0.04 on Chan when building the table.
	return 1.0f;
#elif USE_ENERGY_CONSERVATION == 2
	// TODO
	return 1.f;
#else
	return 1.f;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// BxDFs Energy terms & Energy Preservation/Conservation utils

// Note:
// * For performance reason, we maintain both a *chromatic* and an *achromatic* version of the energy terms.
// * Chromatic and achromatic version are respectively accessible with xxxRGB or xxxA suffix. 
// * USE_ACHROMATIC_BXDF_ENERGY determines which version should be used by default

// Chromatic version
#define BXDF_ENERGY_TYPE float3
#define BXDF_ENERGY_SUFFIX(N) N##RGB
#define IS_BXDF_ENERGY_TYPE_ACHROMATIC 0
#include "ShadingEnergyConservationTemplate.ush"
#undef BXDF_ENERGY_TYPE
#undef BXDF_ENERGY_TYPE_SUFFIX
#undef IS_BXDF_ENERGY_TYPE_ACHROMATIC

// Achromatic version
#define BXDF_ENERGY_TYPE float
#define BXDF_ENERGY_SUFFIX(N) N##A
#define IS_BXDF_ENERGY_TYPE_ACHROMATIC 1
#include "ShadingEnergyConservationTemplate.ush"
#undef BXDF_ENERGY_TYPE
#undef BXDF_ENERGY_TYPE_SUFFIX
#undef IS_BXDF_ENERGY_TYPE_ACHROMATIC

///////////////////////////////////////////////////////////////////////////////////////////////////
// Default implementation

#ifndef USE_ACHROMATIC_BXDF_ENERGY
#define USE_ACHROMATIC_BXDF_ENERGY 0
#endif

#if USE_ACHROMATIC_BXDF_ENERGY
#define FBxDFEnergyType  float 	
#define FBxDFEnergyTerms FBxDFEnergyTermsA
#define ComputeFresnelEnergyTerms ComputeFresnelEnergyTermsA
#define ComputeGGXSpecEnergyTerms ComputeGGXSpecEnergyTermsA
#define ComputeGGXSpecEnergyTerms ComputeGGXSpecEnergyTermsA
#define ComputeClothEnergyTerms   ComputeClothEnergyTermsA
#define ComputeClothEnergyTerms   ComputeClothEnergyTermsA
#define ComputeDiffuseEnergyTerms ComputeDiffuseEnergyTermsA
#else
#define FBxDFEnergyType float3 
#define FBxDFEnergyTerms FBxDFEnergyTermsRGB
#define ComputeFresnelEnergyTerms ComputeFresnelEnergyTermsRGB
#define ComputeGGXSpecEnergyTerms ComputeGGXSpecEnergyTermsRGB
#define ComputeGGXSpecEnergyTerms ComputeGGXSpecEnergyTermsRGB
#define ComputeClothEnergyTerms   ComputeClothEnergyTermsRGB
#define ComputeClothEnergyTerms   ComputeClothEnergyTermsRGB
#define ComputeDiffuseEnergyTerms ComputeDiffuseEnergyTermsRGB
#endif