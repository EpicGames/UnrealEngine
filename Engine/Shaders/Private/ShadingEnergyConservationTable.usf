// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "BRDF.ush"
#include "ShadingCommon.ush"
#include "/Engine/Shared/PathTracingDefinitions.h"

// Move these utility functions into a common/non-path-tracer specific place
#include "PathTracing/Material/PathTracingGlossy.ush"
#include "PathTracing/Material/PathTracingFresnel.ush"
#include "PathTracing/Utilities/PathTracingRandomSequence.ush"

// This shader computes the directional albedo (basically a furnace test) of a given BxDF to determine how much energy is "missing"
// from a given viewing direction and roughness. These tables can later be used to compensate for the missing energy during shading.
// "Energy" here refers to the directional albedo. Energy loss is simply 1-Energy since we assume the raw BxDF should reflect all
// incoming light.

// Number of samples used for integrating directional albedo 
uint NumSamples;
uint EnergyTableResolution;
// x: directional albedo for rough diffuse
RWTexture2D<float>  Output1Texture2D;
// x: directional albedo of pure GGX lobe
// y: directional albedo of GGX+Schlick fresnel term
RWTexture2D<float2> Output2Texture2D;
// x: maximum of the lobe for normalisation of the lob w.r.t. its maximum value.
RWTexture2D<float4> Output3Texture2D;
// x: directional albedo from outside->in, y: directional albedo from inside->out
RWTexture3D<float2> OutputTexture3D;

#if BUILD_ENERGY_TABLE == 4
groupshared float MaxLobeValue[THREADGROUPSIZE_X][THREADGROUPSIZE_Y][4];
[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 4)]
#else
[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, 1)]
#endif
void BuildEnergyTableCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint3 GroupThreadId : SV_GroupThreadID)
{
	const float3 Index = (float3(DispatchThreadId) + 0.5) / float(EnergyTableResolution);

#if BUILD_ENERGY_TABLE == 4
	// In this case we want exact uvs [0.5/height, (height-0.5)/height] to map to roughness [0, 1] to catch the extrems.
	// This approach could potentially be applied to all the LUTs.
	const float Roughness  =     float(DispatchThreadId.x) / float(EnergyTableResolution - 1);
	const float Anisotropy =     float(DispatchThreadId.y) / float(EnergyTableResolution - 1);		// in [0,1] as abs(aniso)
	const float NoV        = max(float(DispatchThreadId.z) / float(4                     - 1), 1e-8f);
#else	// Map from thread ID to the exact roughness/viewing angle we want to compute
	const float NoV = max(Index.x, 1e-8);
	const float Roughness =  Index.y;
#endif

#if BUILD_ENERGY_TABLE == 1
	const float EtaIn = 1.0 + 2.0 * Index.z; // only handle Ior in [1,3] range
	const float EtaOut = rcp(EtaIn);
	// NOTE: Baking a table for all possible F0 values would greatly complicate things
	// Instead, we aim for optimal results when the Ior values is consistent with F0.
	// In practice, it appears that this gives very good results, and the penalty when
	// Ior and F0 are mismatched does not appear to be that large.
	const float F0 = DielectricIorToF0(EtaIn); // NOTE: this is always equal to DielectricIorToF0(EtaOut)
#endif

	float2 Alpha; GetGGXBasis(Roughness, float3(0, 0, 1), Alpha);
	float3 V = float3(sqrt(1 - NoV * NoV), 0, NoV);

	float2 Result = 0;
	// use plenty of samples -- this is computed on startup once and at fairly low resolution
	LOOP
	for (int Sample = 0; Sample < NumSamples; Sample++)
	{
		// draw a random sample
		float3 RandSample = RandomSequenceCreate(DispatchThreadId, Sample, NumSamples).Get3D();

		float2 Weight = 0;

#if BUILD_ENERGY_TABLE == 0
		const float3 H = ImportanceSampleVisibleGGX(RandSample.xy, Alpha, V).xyz;
		const float3 L = reflect(-V, H);
		if (L.z > 0)
		{
			const float2 GGXResult = GGXEvalReflection(L, V, H, Alpha);
			Weight = float2(GGXResult.x, GGXResult.x * Pow5(1 - dot(V, H)));
		}
#elif BUILD_ENERGY_TABLE == 4
#if 0
		const float3 H = ImportanceSampleVisibleGGX(RandSample.xy, Alpha, V).xyz;

		const float3 L = reflect(-V, H);
		if (L.z > 0)
		{
			const float a2 = Alpha.x * Alpha.x;	// no anisotropy
			const float NoH = H.z;
			const float NoL = L.z;
			Weight = D_GGX(a2, NoH); // Isotropic lobe
		}
#else
		float2 AlphaXY = 0;
		GetAnisotropicRoughness(Alpha.x, Anisotropy, AlphaXY.x, AlphaXY.y);
		const float3 H = ImportanceSampleVisibleGGX(RandSample.xy, AlphaXY, V, false).xyz;
		const float3 L = reflect(-V, H);
		if (L.z > 0)
		{
			Weight = max(0, D_GGXaniso(AlphaXY.x, AlphaXY.y, H.z, H.x, H.y));// Anisotropic lobe
		}
#endif
#elif BUILD_ENERGY_TABLE == 1
		const float3 H = ImportanceSampleVisibleGGX(RandSample.xy, Alpha, V, false).xyz;
		{
			// compute weight for outside -> in case
			float3 L = 0;
			float F = 0;
			if (SampleRefraction(-V, H, EtaIn, F0, RandSample.z, L, F))
			{
				Weight.x = GGXEvalRefraction(L, V, H, Alpha, EtaIn).x;
			}
			else
			{
				Weight.x = GGXEvalReflection(L, V, H, Alpha, false).x;
			}
		}
		{
			// compute weight for inside -> out case
			float3 L = 0;
			float F = 0;
			if (SampleRefraction(-V, H, EtaOut, F0, RandSample.z, L, F))
			{
				Weight.y = GGXEvalRefraction(L, V, H, Alpha, EtaOut).x;
			}
			else
			{
				Weight.y = GGXEvalReflection(L, V, H, Alpha, false).x;
			}
		}
#elif BUILD_ENERGY_TABLE == 2
		const float3 L = CosineSampleHemisphere(RandSample.xy).xyz;
		const float NoV = V.z;
		const float NoL = L.z;
		const float3 H = normalize(V + L);
		const float NoH = H.z;
#if SUBSTRATE_ENABLED
		const float DCloth = D_Charlie(Roughness, NoH);
		const float VCloth = Vis_Ashikhmin(NoV, NoL);
#else
		const float DCloth = D_InvGGX(Alpha.x * Alpha.y, NoH);
		const float VCloth = Vis_Cloth(NoV, NoL);
#endif
		const float ClothWeight = NoL > 0 && NoV > 0 ? (PI * DCloth * VCloth) : 0.0;

		Weight = float2(ClothWeight, ClothWeight * Pow5(1 - dot(V, H)));
#elif BUILD_ENERGY_TABLE == 3
		const float3 L = CosineSampleHemisphere(RandSample.xy).xyz;
		const float NoV = V.z;
		const float NoL = L.z;
		const float3 H = normalize(V + L);
		const float VoH = dot(V, H);
		const float NoH = H.z;
		const float DiffuseWeight = NoL > 0 && NoV > 0 ? PI * Diffuse_GGX_Rough(float3(1, 1, 1), Roughness, NoV, NoL, VoH, NoH, 1.0f).x : 0.0f;

		Weight = float2(DiffuseWeight, 1.f);
#endif

#if BUILD_ENERGY_TABLE == 4
		// Max of all samples
		Result = max(Result, Weight.x);
#else
		// average in
		Result = lerp(Result, Weight, 1.0 / (Sample + 1));
#endif
	}

#if BUILD_ENERGY_TABLE == 0 || BUILD_ENERGY_TABLE == 2
	Output2Texture2D[DispatchThreadId.xy] = Result;
#endif

#if BUILD_ENERGY_TABLE == 1
	OutputTexture3D[DispatchThreadId] = Result;
#endif

#if BUILD_ENERGY_TABLE == 3
	Output1Texture2D[DispatchThreadId.xy] = Result.x;
#endif

#if BUILD_ENERGY_TABLE == 4

	MaxLobeValue[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Result.x;
	GroupMemoryBarrierWithGroupSync();

	BRANCH
	if (DispatchThreadId.z == 0)
	{
		// Take the max of the 4 NoV samples for this roughness/anisotropy and store it in the LUT. This will be used for energy compensation in toon shading to make sure the specular lobe can reach 1.0 at its peak.
		const float4 MaxLobeValues = float4(
			MaxLobeValue[GroupThreadId.x][GroupThreadId.y][0],
			MaxLobeValue[GroupThreadId.x][GroupThreadId.y][1],
			MaxLobeValue[GroupThreadId.x][GroupThreadId.y][2],
			MaxLobeValue[GroupThreadId.x][GroupThreadId.y][3]);

		Output3Texture2D[DispatchThreadId.xy] = saturate(GGX_MAX_LUT_STORE_SCALE * (1.0f / MaxLobeValues));
	}
#endif
}
