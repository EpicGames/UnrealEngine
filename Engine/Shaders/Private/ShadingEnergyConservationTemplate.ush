// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef BXDF_ENERGY_TYPE
#error  BXDF_ENERGY_TYPE needs to be defined
#endif

#ifndef BXDF_ENERGY_SUFFIX
#error  BXDF_ENERGY_SUFFIX needs to be defined
#endif

#ifndef IS_BXDF_ENERGY_TYPE_ACHROMATIC
#error  IS_BXDF_ENERGY_TYPE_ACHROMATIC needs to be defined
#endif

#include "ShadingCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Energy terms

BXDF_ENERGY_TYPE BXDF_ENERGY_SUFFIX(GetF0F90)(float3 InF0)
{
#if IS_BXDF_ENERGY_TYPE_ACHROMATIC
	return max3(InF0.x, InF0.y, InF0.z);
#else
	return InF0;
#endif
}

float GetEnergyLuminance(BXDF_ENERGY_TYPE InE)
{
#if IS_BXDF_ENERGY_TYPE_ACHROMATIC
	return InE;
#else
	return Luminance(InE);
#endif
}

struct BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms)
{
	BXDF_ENERGY_TYPE W; // overall weight to scale the lobe BxDF by to ensure energy conservation
	BXDF_ENERGY_TYPE E; // Directional albedo of the lobe for energy preservation and lobe picking
};

// Given a split-sum approximation of directional albedo for a BxDF, compute multiple scattering weight and multiple scattering directional albedo
// while taking into account the fresnel term (assumed to be F_Schlick)
BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) BXDF_ENERGY_SUFFIX(ComputeFresnelEnergyTerms)(float2 E, float3 InF0, float3 InF90)
{
	BXDF_ENERGY_TYPE F0  = BXDF_ENERGY_SUFFIX(GetF0F90)(InF0);
	BXDF_ENERGY_TYPE F90 = BXDF_ENERGY_SUFFIX(GetF0F90)(InF90);

	BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) Result;
	// [2] Eq 16: this restores the missing energy of the bsdf, while also accounting for the fact that the fresnel term causes some energy to be absorbed
	// NOTE: using F0 here is an approximation, but for schlick fresnel Favg is almost exactly equal to F0
#if USE_DEVELOPMENT_SHADERS
	Result.W = View.bShadingEnergyConservation ? (1.0 + F0 * ((1 - E.x) / E.x)) : 1.0f;
#else
	Result.W = 1.0 + F0 * ((1 - E.x) / E.x);
#endif
	// Now estimate the amount of energy reflected off this specular lobe so that we can remove it from underlying BxDF layers (like diffuse)
	// This relies on the split-sum approximation as in [3] Sec 4.
	// This term can also be useful to compute the probability of choosing among lobes
	Result.E = Result.W * (E.x * F0 + E.y * (F90 - F0));
	return Result;
}

BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) BXDF_ENERGY_SUFFIX(ComputeGGXSpecEnergyTerms)(float Roughness, float NoV, float3 F0, float3 F90)
{
	BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) Out;
#if USE_ENERGY_CONSERVATION > 0
	{
		Out = BXDF_ENERGY_SUFFIX(ComputeFresnelEnergyTerms)(GGXEnergyLookup(Roughness, NoV), F0, F90);
}
#else
	{
		Out.W = 1.0f;
		Out.E = BXDF_ENERGY_SUFFIX(GetF0F90)(F0);
	}
#endif
	return Out;
}

BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) BXDF_ENERGY_SUFFIX(ComputeGGXSpecEnergyTerms)(float Roughness, float NoV, float3 F0)
{
	const float F90 = F0RGBToMicroOcclusion(F0);
	return BXDF_ENERGY_SUFFIX(ComputeGGXSpecEnergyTerms)(Roughness, NoV, F0, F90);
}

BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) BXDF_ENERGY_SUFFIX(ComputeClothEnergyTerms)(float Roughness, float NoV)
{
	BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) Out;
#if USE_ENERGY_CONSERVATION > 0
	{
		Out.W = 1.f;
		Out.E = ClothEnergyLookup(Roughness, NoV).x;
	}
#else
	{
		Out.W = 1.0f;
		Out.E = 1.0f;
	}
#endif
	return Out;
}

BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) BXDF_ENERGY_SUFFIX(ComputeDiffuseEnergyTerms)(float Roughness, float NoV)
{
	BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) Out;
#if USE_ENERGY_CONSERVATION > 0
	{
		Out.E = DiffuseEnergyLookup(Roughness, NoV);
	}
#else
	{
		Out.E = 1.0f;
	}
#endif
	Out.W = 1.0f;
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Energy Preservation/Conservation

// Return the energy absorbed by upper layer (e.g., for the specular layer attenuation onto diffuse)
// Note: Use the directional albedo luminance to avoid color-shift due to metallic specular (for which the energy should be absorbed, not transmitted)
float ComputeEnergyPreservation(BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) EnergyTerms)
{
#if USE_ENERGY_CONSERVATION > 0
#if USE_DEVELOPMENT_SHADERS
	return View.bShadingEnergyPreservation ? (1 - GetEnergyLuminance(EnergyTerms.E)) : 1.0f;
#else
	return 1 - GetEnergyLuminance(EnergyTerms.E);
#endif
#else
	return 1.0f;
#endif
}

// Return the energy conservation weight factor for account energy loss in the BSDF model (i.e. due to micro-facet multiple scattering)
BXDF_ENERGY_TYPE ComputeEnergyConservation(BXDF_ENERGY_SUFFIX(FBxDFEnergyTerms) EnergyTerms)
{
	return EnergyTerms.W;
}

#undef BXDF_ENERGY_SUFFIX