// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#include "LargeWorldCoordinates.ush" 

#if SUBSTRATE_ENABLED
// In order to avoid numerical issue with integrating low roughness value, incease the min. GGX roughness
#include "/Engine/Shared/SubstrateDefinitions.h"
#ifdef SUBSTRATE_MIN_GGX_ROUGHNESS
#undef SUBSTRATE_MIN_GGX_ROUGHNESS
#define SUBSTRATE_MIN_GGX_ROUGHNESS 0.04
#endif
#endif

#define SUPPORT_CONTACT_SHADOWS 0
#define SHADING_PATH_DEFERRED 1
#define USE_SOURCE_TEXTURE 0
#define DEBUG_ENABLE 0
#define SUBSTRATE_SSS_TRANSMISSION USE_TRANSMISSION
#if SUBSTRATE_GBUFFER_FORMAT==0
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 0	// Load from a specified BSDF, not from MAterialBuffer.
#define SUBSTRATE_INLINE_SHADING 1				// Inline shading for normal setup if needed.
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 1		// Need for SSS dither.
#include "Substrate/SubstrateRead.ush"
#else 
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 1
#endif

#include "Substrate/Substrate.ush"
#include "DeferredShadingCommon.ush"
#include "DeferredLightingCommon.ush"
#include "MonteCarlo.ush" 

#if DEBUG_ENABLE
#include "ShaderPrint.ush"
#include "ColorMap.ush"
#endif 

#include "Substrate/SubstrateEvaluation.ush"
#include "Substrate/SubstrateDeferredLighting.ush"
#include "Substrate/SubstrateRead.ush"

#if SHADER_FURNACE_ANALYTIC

uint IntegratorType;
uint NumSamplesPerSet;

void MainPS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	// Non Substrate path (todo: add Substrate path)
	const float2 ScreenUV = SvPositionToBufferUV(SVPos);
	const float2 ScreenPosition = SvPositionToScreenPosition(SVPos).xy;
	const float2 PixelPos = SVPos.xy;

	const FRectTexture RectTexture = InitRectTexture();
	const float Dither = InterleavedGradientNoise(PixelPos, View.StateFrameIndexMod8);

	bool bIsValid = false;
	float Roughness = 0.5f;
	float ClearCoatRoughness = 0.5f;
	float3 N = float3(0, 0, 1);
	float SceneDepth = 0;
#if SUBSTRATE_ENABLED
	{
		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
		// Fetch the first layer of roughness & normal only. This fine for simple material but will not work for clear coat.
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		bIsValid = SubstratePixelHeader.ClosureCount > 0;
		#else
		FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(ScreenUV, true);
		bIsValid = ScreenSpaceData.GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
		#endif
		if (bIsValid)
		{
			const float3 DummyV = float3(0, 0, 1);
			const float3 DummyL = float3(0, 0, 1);
			#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
			FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
			FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, DummyV, DummyL);
			#else
			FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
			FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(GBufferBSDF.BSDF, GBufferBSDF.BSDFTangentBasis, DummyV, DummyL);
			#endif

			SceneDepth = CalcSceneDepth(ScreenUV);
			N = BSDFContext.N;
			Roughness = MakeRoughnessSafe(SubstrateGetBSDFRoughness(BSDFContext.BSDF), SUBSTRATE_MIN_GGX_ROUGHNESS);

			ClearCoatRoughness = Roughness;
			if (BSDF_GETTYPE(BSDFContext.BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && BSDF_GETHASHAZINESS(BSDFContext.BSDF))
			{
				const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
				BRANCH
				if (Haziness.bSimpleClearCoat)
				{
					ClearCoatRoughness = MakeRoughnessSafe(Haziness.Roughness, SUBSTRATE_MIN_GGX_ROUGHNESS);
				}
			}
		}
	}
#else
	FScreenSpaceData ScreenSpaceData;
	{
		ScreenSpaceData = GetScreenSpaceData(ScreenUV);
		ScreenSpaceData.AmbientOcclusion = 1.0f;
		ScreenSpaceData.GBuffer.Roughness = max(ScreenSpaceData.GBuffer.Roughness, View.MinRoughness* 2.f);

		N = ScreenSpaceData.GBuffer.WorldNormal;
		SceneDepth = ScreenSpaceData.GBuffer.Depth;

		bIsValid = SceneDepth > 0 && ScreenSpaceData.GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
		Roughness = ScreenSpaceData.GBuffer.Roughness;
		const bool bIsClearCoat = ScreenSpaceData.GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT;
		ClearCoatRoughness = bIsClearCoat ? max(ScreenSpaceData.GBuffer.CustomData.y, 0.02f) : ScreenSpaceData.GBuffer.Roughness;
	
	}
#endif

	const float4 PixelLightAttenuation = 1.0f;
	const float3 InRadiance = 0.5f;
	const float3 WorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 CameraVector = normalize(WorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
	const float3 TranslatedWorldPosition = WorldPosition;
	const float3 V = -CameraVector;

#if DEBUG_ENABLE
	const float3x3 TangentBasis = GetTangentBasis(N);
	const uint2 CursorCoord = GetCursorPos();
	const bool bDebugEnabled = all(CursorCoord == uint2(PixelPos));
	if (bDebugEnabled)
	{
		AddReferentialTWS(WorldPosition, TangentBasis[0], TangentBasis[1], TangentBasis[2], 3.f);
		AddLineTWS(WorldPosition, WorldPosition + V * 3.0f, ColorPurple);
	}
#endif

	if (bIsValid)
	{
		const uint NumSets = 3;
		const uint AllocNumSets = 3;
		const uint NumSamples[AllocNumSets] =
		{
			NumSamplesPerSet,	// GGX
			NumSamplesPerSet,	// Cosine hemisphere
			NumSamplesPerSet,	// GGX coat
		};

		float3 Acc = 0.f;
		float3 FinalRadiance = 0.f;

		// Set loop (Cosine/GGX/... distribuciton)
		LOOP
		for (uint Set = 0; Set < NumSets; Set++)
		{
			// Samples loop
			LOOP
			for (uint SampleIt = 0; SampleIt < NumSamples[Set]; ++SampleIt)
			{
				const float2 E = Hammersley(SampleIt, NumSamples[Set], 0);

				// Sample generation
				float3 L = 0, H = 0;
				if (Set == 0)
				{
					H = TangentToWorld(ImportanceSampleGGX(E, Pow4(Roughness)).xyz, N);
					L = 2 * dot(V, H) * H - V;
				}
				else if (Set == 1)
				{
					L = TangentToWorld(CosineSampleHemisphere(E).xyz, N);
					H = normalize(V + L);
				}
				else if (Set == 2)
				{
					H = TangentToWorld(ImportanceSampleGGX(E, Pow4(ClearCoatRoughness)).xyz, N);
					L = 2 * dot(V, H) * H - V;
				}

				const float NoL = saturate(dot(N, L));
				const float NoH = saturate(dot(N, H));
				const float VoH = saturate(dot(V, H));

				// PDFs
				float PDFs[] =
				{
					NoH > 0 && VoH > 0 ? D_GGX(Pow4(Roughness), NoH) * NoH / (4 * VoH) : 0.f,
					NoL / PI,
					NoH > 0 && VoH > 0 ? D_GGX(Pow4(ClearCoatRoughness), NoH) * NoH / (4 * VoH) : 0.f,
				};
				
				// MIS power heuristic
				float InvWeight = 0;
				UNROLL for (uint j = 0; j < NumSets; j++)
				{
					InvWeight += Square(PDFs[j] * NumSamples[j]);
				}
				float Weight = (InvWeight > 1e-5f ? rcp(InvWeight) : 0.f) * PDFs[Set] * NumSamples[Set];

				#if DEBUG_ENABLE
				if (bDebugEnabled)
				{
					const float Scale = Set == 0 ? 1.f : 2.0f;
					const float4 DebugColor = Set == 0 ? ColorOrange : ColorEmerald;
					AddLineTWS(WorldPosition, WorldPosition + L * Scale, DebugColor);
				}
				#endif

				FDeferredLightData LightData;
				{
					LightData.TranslatedWorldPosition	= float3(0,0,0);  // Directional light
					LightData.InvRadius					= 0.f;			  // Directional light
					LightData.Color						= InRadiance;
					LightData.FalloffExponent			= 0.f;			  // Directional light
					LightData.Direction					= L;
					LightData.Tangent					= float3(1,0,0);
					LightData.SpotAngles				= 0.f;
					LightData.SourceRadius				= 0.f;
					LightData.SourceLength				= 0.f;
					LightData.SoftSourceRadius			= 0.f;
					LightData.SpecularScale				= 1.0f;
					LightData.DiffuseScale				= 1.0f;
					LightData.ContactShadowLength		= 0.f;
					LightData.ContactShadowLengthInWS	= 0.f;
					LightData.ContactShadowCastingIntensity = 1.f;
					LightData.ContactShadowNonCastingIntensity = 0.f;
					LightData.DistanceFadeMAD			= 0.f;
					LightData.ShadowMapChannelMask		= 0;
					LightData.ShadowedBits				= 0;
					LightData.bInverseSquared			= false;  // Directional light
					LightData.bRadialLight				= false;
					LightData.bSpotLight				= false;
					LightData.bRectLight				= false;
					LightData.RectLightData.BarnCosAngle= 0.f;
					LightData.RectLightData.BarnLength	= 0.f;
					LightData.RectLightData.AtlasData.AtlasUVOffset	= 0.f;
					LightData.RectLightData.AtlasData.AtlasUVScale	= 0.f;
					LightData.RectLightData.AtlasData.AtlasMaxLevel	= 0.f;
					LightData.HairTransmittance			= InitHairTransmittanceData();
					LightData.ShadowedBits				= 0;
				}

				float3 SampleRadiance = 0;
			#if !SUBSTRATE_ENABLED
				{
					float SurfaceShadow = 1.0f;
					SampleRadiance = GetDynamicLighting(
						TranslatedWorldPosition, 
						CameraVector, 
						ScreenSpaceData.GBuffer, 
						ScreenSpaceData.AmbientOcclusion, 
						LightData, 
						PixelLightAttenuation,
						Dither, 
						uint2(PixelPos), 
						SurfaceShadow).xyz;
				}
			#else
				{

				#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
					FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
					FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
					const bool bMaterialIsLit = SubstratePixelHeader.ClosureCount > 0;	// This test is also enough to exclude sky pixels
					#if SUBSTRATE_INLINE_SHADING
					HEADER_SETIRRADIANCE_AO(SubstratePixelHeader.State, SubstratePackIrradianceAndOcclusion(InitIrradianceAndOcclusion(1.f));
					#endif
				#else
					FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(ScreenUV, true);
					ScreenSpaceData.AmbientOcclusion = 1.f;
					const FGBufferData GBuffer = ScreenSpaceData.GBuffer;
					const bool bMaterialIsLit = GBuffer.ShadingModelID != SHADINGMODELID_UNLIT;
				#endif

					const float LightMask = 1;

					BRANCH
					if (bMaterialIsLit)
					{

						#if SUBSTRATE_GBUFFER_FORMAT==0
						FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
						#endif

						FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters	= GetInitialisedSubstrateShadowTermInputParameters();
						SubstrateShadowTermInputParameters.bEvaluateShadowTerm					= false;
						FSubstrateDeferredLighting SampleLighting = SubstrateDeferredLighting(
							LightData,
							V,
							L,
							L,
							LightMask,
							SubstrateShadowTermInputParameters,
							#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
							Substrate.MaterialTextureArray,
							SubstrateAddressing,
							SubstratePixelHeader
							#else
							PixelPos,
							GBufferBSDF.BSDF,
							GBufferBSDF.BSDFTangentBasis,
							GBufferBSDF.BSDFAO
							#endif
						);
						SampleRadiance.xyz = SampleLighting.TotalDiffuseLighting + SampleLighting.TotalSpecularLighting;
					}
				}
			#endif

				FinalRadiance += Weight * SampleRadiance;
			}
			OutColor = float4(FinalRadiance, 1.f);
		}
	}
	else
	{
		OutColor = float4(InRadiance, 1.f);
	}

}

#endif // SHADER_FURNACE_ANALYTIC

////////////////////////////////////////////////////////////////////////////////////////////////////////////
