// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DeferredShadingCommon.ush"
#include "BRDF.ush"
#include "FastMath.ush"
#include "CapsuleLight.ush"
#include "RectLight.ush"
#include "AreaLightCommon.ush"
#include "TransmissionCommon.ush"
#include "HairBsdf.ush"
#include "ShadingEnergyConservation.ush"
#include "ParticipatingMediaCommon.ush"
#include "ColorSpace.ush"
#include "SGGX.ush"
#include "PathTracing/Utilities/PathTracingRandomSequence.ush"

#ifndef HAIR_BSDF_BACKLIT
#define HAIR_BSDF_BACKLIT 1
#endif

#if SHADING_PATH_MOBILE
#include "MobileGGX.ush"

#ifndef MOBILE_SHADOW_QUALITY
#define MOBILE_SHADOW_QUALITY 2
#endif

//It's always 0 in mobile deferred lighting shaders
#ifndef FULLY_ROUGH
#define FULLY_ROUGH 0
#endif

//It's always 0 in mobile deferred lighting shaders
#ifndef NONMETAL
#define NONMETAL 0
#endif

//It's always 0 in mobile deferred lighting shaders
#ifndef MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#define MATERIAL_SHADINGMODEL_SINGLELAYERWATER 0
#endif

// Always enabled in mobile deferred lighting shaders
#ifndef MOBILE_USE_PREINTEGRATED_GF
#define MOBILE_USE_PREINTEGRATED_GF 1
#endif

//It's always 1 in mobile deferred lighting shaders
#ifndef MOBILE_DEFERRED_LIGHTING
#define MOBILE_DEFERRED_LIGHTING 0
#endif

#ifndef DEFERRED_SHADING_PATH
#define DEFERRED_SHADING_PATH 0
#endif

/*------------------------------------------------------------------------------
	Mobile Shading Model Functions
------------------------------------------------------------------------------*/
half4 GetSSProfilePreIntegratedValue(uint SubsurfaceProfileInt, half NoL, half Curvature)
{
	float3 UV = float3((NoL * .5 + .5), Curvature, SubsurfaceProfileInt);

	return Texture2DArraySampleLevel(View.SSProfilesPreIntegratedTexture, View.SSProfilesPreIntegratedSampler, UV, 0);
}

half3 GetEnvBRDF(half3 SpecularColor, half Roughness, half NoV)
{
#if FULLY_ROUGH
	return 0.0f;
#elif MOBILE_USE_PREINTEGRATED_GF
	return EnvBRDF(SpecularColor, Roughness, NoV);
#elif NONMETAL
	// If nothing is hooked up to Metalic and Specular,
	// then defaults are the same as a non-metal,
	// so this define is safe.
	return EnvBRDFApproxNonmetal(Roughness, NoV).xxx;
#else
	return EnvBRDFApprox(SpecularColor, Roughness, NoV);
#endif
}
#endif // SHADING_PATH_MOBILE

struct FDirectLighting
{
	float3	Diffuse;
	float3	Specular;
	float3	Transmission;
};

struct FShadowTerms
{
	// This is the surface shadow from opaque.
	half	SurfaceShadow;

	// This is the surface shadow from opaque multiplied by transmittance from subsurface elements.
	half	TransmissionShadow;

	// This can be OpticalThickness or Transmittance depending (see ShadowProjectionPixelShader.usf and GetShadowTermsBase(...)).
	// Shading model - Subsurface			=> Transmittance in [0, 1]
	// Shading model - Subsurface Profile	=> Encoded OpticalThickness to be decoded using DecodeOpticalDepthFromShadowMask.
	// Substrate	 - Per Pixel MFP		=> Same as Subsurface Profile.
	half	TransmittanceOrOpticalThickness;

	FHairTransmittanceData HairTransmittance;
};

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting HairBxDF(FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow)
{
	const float3 BsdfValue = HairShading(GBuffer, AreaLight.DiffuseL, V, N, Shadow.TransmissionShadow, Shadow.HairTransmittance, HAIR_BSDF_BACKLIT, 0, uint2(0, 0));

	FDirectLighting Lighting;
	Lighting.Diffuse = 0;
	Lighting.Specular = 0;
	Lighting.Transmission = AreaLight.FalloffColor * AreaLight.Falloff * BsdfValue;
	return Lighting;
}

float New_a2( float a2, float SinAlpha, float VoH )
{
	return a2 + 0.25 * SinAlpha * (3.0 * sqrtFast(a2) + SinAlpha) / ( VoH + 0.001 );
	//return a2 + 0.25 * SinAlpha * ( saturate(12 * a2 + 0.125) + SinAlpha ) / ( VoH + 0.001 );
	//return a2 + 0.25 * SinAlpha * ( a2 * 2 + 1 + SinAlpha ) / ( VoH + 0.001 );
}

float EnergyNormalization( inout float a2, float VoH, FAreaLight AreaLight )
{
	if( AreaLight.SphereSinAlphaSoft > 0 )
	{
		// Modify Roughness
		a2 = saturate( a2 + Pow2( AreaLight.SphereSinAlphaSoft ) / ( VoH * 3.6 + 0.4 ) );
	}

	float Sphere_a2 = a2;
	float Energy = 1;
	if( AreaLight.SphereSinAlpha > 0 )
	{
		Sphere_a2 = New_a2( a2, AreaLight.SphereSinAlpha, VoH );
		Energy = a2 / Sphere_a2;
	}

	if( AreaLight.LineCosSubtended < 1 )
	{
#if 1
		float LineCosTwoAlpha = AreaLight.LineCosSubtended;
		float LineTanAlpha = sqrt( ( 1.0001 - LineCosTwoAlpha ) / ( 1 + LineCosTwoAlpha ) );
		float Line_a2 = New_a2( Sphere_a2, LineTanAlpha, VoH );
		Energy *= sqrt( Sphere_a2 / max(Line_a2, 1e-5) );
#else
		float LineCosTwoAlpha = AreaLight.LineCosSubtended;
		float LineSinAlpha = sqrt( 0.5 - 0.5 * LineCosTwoAlpha );
		float Line_a2 = New_a2( Sphere_a2, LineSinAlpha, VoH );
		Energy *= Sphere_a2 / max(Line_a2, 1e-5);
#endif
	}

	return Energy;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
float3 SpecularGGX(float Roughness, float Anisotropy, float3 SpecularColor, BxDFContext Context, FAreaLight AreaLight)
{
	float Alpha = Roughness * Roughness;
	float ax = 0;
	float ay = 0;
	GetAnisotropicRoughness(Alpha, Anisotropy, ax, ay);

	// Generalized microfacet specular
	float3 D = D_GGXaniso(ax, ay, Context.NoH, Context.XoH, Context.YoH);
	float3 Vis = Vis_SmithJointAniso(ax, ay, Context.NoV, AreaLight.NoL, Context.XoV, Context.XoL, Context.YoV, Context.YoL);
	float3 F = F_Schlick( SpecularColor, Context.VoH );

	return (D * Vis) * F;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
float3 SpecularGGX( float Roughness, float3 SpecularColor, BxDFContext Context, FAreaLight AreaLight )
{
	float a2 = Pow4( Roughness );
	float Energy = EnergyNormalization( a2, Context.VoH, AreaLight );
	
	// Generalized microfacet specular
	float D = D_GGX( a2, Context.NoH ) * Energy;
	float Vis = Vis_SmithJointApprox( a2, Context.NoV, AreaLight.NoL );
	float3 F = F_Schlick( SpecularColor, Context.VoH );

	return (D * Vis) * F;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
half3 DualSpecularGGX(half AverageRoughness, half Lobe0Roughness, half Lobe1Roughness, half LobeMix, half3 SpecularColor, BxDFContext Context, FAreaLight AreaLight)
{
	float AverageAlpha2 = Pow4(AverageRoughness);
	float Lobe0Alpha2 = Pow4(Lobe0Roughness);
	float Lobe1Alpha2 = Pow4(Lobe1Roughness);

	float Lobe0Energy = EnergyNormalization(Lobe0Alpha2, Context.VoH, AreaLight);
	float Lobe1Energy = EnergyNormalization(Lobe1Alpha2, Context.VoH, AreaLight);

	// Generalized microfacet specular
	float D = lerp(D_GGX(Lobe0Alpha2, Context.NoH) * Lobe0Energy, D_GGX(Lobe1Alpha2, Context.NoH) * Lobe1Energy, LobeMix);
	float Vis = Vis_SmithJointApprox(AverageAlpha2, Context.NoV, AreaLight.NoL); // Average visibility well approximates using two separate ones (one per lobe).
	float3 F = F_Schlick(SpecularColor, Context.VoH);

	return (D * Vis) * F;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting DefaultLitBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	BxDFContext Context;
	FDirectLighting Lighting;
	Lighting.Diffuse = 0;
	Lighting.Specular = 0;
	Lighting.Transmission = 0;

	BRANCH
	if (AreaLight.NoL > 0.0f)
	{
#if SUPPORTS_ANISOTROPIC_MATERIALS
		bool bHasAnisotropy = HasAnisotropy(GBuffer.SelectiveOutputMask);
#else
		bool bHasAnisotropy = false;
#endif

		float NoV, VoH, NoH;
		BRANCH
		if (bHasAnisotropy)
		{
			half3 X = GBuffer.WorldTangent;
			half3 Y = normalize(cross(N, X));
			Init(Context, N, X, Y, V, AreaLight.SpecularL);

			NoV = Context.NoV;
			VoH = Context.VoH;
			NoH = Context.NoH;
		}
		else
		{
			Init(Context, N, V, AreaLight.SpecularL);
			
			NoV = Context.NoV;
			VoH = Context.VoH;
			NoH = Context.NoH;

			SphereMaxNoH(Context, AreaLight.SphereSinAlpha, true);
		}

		Context.NoV = saturate(abs( Context.NoV ) + 1e-5);

#if MATERIAL_ROUGHDIFFUSE
		// Diffuse model with roughness == specular roughness. This is not necessarily a good model of reality because it assumes microfacets with lambertian response.
		// The limit behavior of SSS as the mean free path gets small behaves differently, but this is a start.
		// Also we cannot use the morphed context maximising NoH as this is causing visual artefact when interpolating rough/smooth diffuse response. (SUBSTRATE_TODO: Check this again with latest rough diffuse model)
		Lighting.Diffuse = Diffuse_GGX_Rough(GBuffer.DiffuseColor, GBuffer.Roughness, NoV, AreaLight.NoL, VoH, NoH, GetAreaLightDiffuseMicroReflWeight(AreaLight));
#else
		Lighting.Diffuse = Diffuse_Lambert(GBuffer.DiffuseColor);
#endif
		Lighting.Diffuse *= AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL);

		BRANCH
		if (bHasAnisotropy)
		{
			Lighting.Specular = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * SpecularGGX(GBuffer.Roughness, GBuffer.Anisotropy, GBuffer.SpecularColor, Context, AreaLight);
		}
		else
		{
			if( IsRectLight(AreaLight) )
			{
				Lighting.Specular = RectGGXApproxLTC(GBuffer.Roughness, GBuffer.SpecularColor, N, V, AreaLight.Rect, AreaLight.Texture);
			}
			else
			{
				Lighting.Specular = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * SpecularGGX(GBuffer.Roughness, GBuffer.SpecularColor, Context, AreaLight);
			}
		}

		FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, Context.NoV, GBuffer.SpecularColor);

		// Add energy presevation (i.e. attenuation of the specular layer onto the diffuse component
		Lighting.Diffuse *= ComputeEnergyPreservation(EnergyTerms);

		// Add specular microfacet multiple scattering term (energy-conservation)
		Lighting.Specular *= ComputeEnergyConservation(EnergyTerms);

		Lighting.Transmission = 0;
	}

	return Lighting;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
// Simple default lit model used by Lumen
float3 SimpleShading( float3 DiffuseColor, float3 SpecularColor, float Roughness, float3 L, float3 V, half3 N )
{
	const float NoV = saturate(dot(N, V));
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(Roughness, NoV, SpecularColor);

	float3 H = normalize(V + L);
	float NoH = saturate( dot(N, H) );
	
	// Generalized microfacet specular
	float D = D_GGX( Pow4(Roughness), NoH );
	float Vis = Vis_Implicit();
	float3 F = F_None( SpecularColor );

	return 
		Diffuse_Lambert( DiffuseColor ) * ComputeEnergyPreservation(EnergyTerms) +
		(D * Vis) * F * ComputeEnergyConservation(EnergyTerms);
}

float RefractBlend(float VoH, float Eta)
{
	// Refraction blend factor for normal component of VoH
	float k = 1.0 - Eta * Eta * (1.0 - VoH * VoH);
	return Eta * VoH - sqrt(k);
}

half RefractBlendClearCoatApprox(half VoH)
{
	// Polynomial approximation of refraction blend factor for normal component of VoH with fixed Eta (1/1.5):
	return (0.63 - 0.22 * VoH) * VoH - 0.745;
}

float3 Refract(float3 V, float3 H, float Eta)
{
	// Assumes V points away from the point of incidence
	float VoH = dot(V, H);
	return RefractBlend(VoH, Eta) * H - Eta * V;
}

BxDFContext RefractClearCoatContext(BxDFContext Context)
{
	// Reference: Propagation of refraction through dot-product NoV
	// Note: This version of Refract requires V to point away from the point of incidence
	//  NoV2 = -dot(N, Refract(V, H, Eta))
	//  NoV2 = -dot(N, RefractBlend(VoH, Eta) * H - Eta * V)
	//  NoV2 = -(RefractBlend(VoH, Eta) * NoH - Eta * NoV)
	//  NoV2 = Eta * NoV - RefractBlend(VoH, Eta) * NoH
	//  NoV2 = 1.0 / 1.5 * NoV - RefractBlendClearCoatApprox(VoH) * NoH

	BxDFContext RefractedContext = Context;
	half Eta = 1.0 / 1.5;
	half RefractionBlendFactor = RefractBlendClearCoatApprox(Context.VoH);
	half RefractionProjectionTerm = RefractionBlendFactor * Context.NoH;
	RefractedContext.NoV = clamp(Eta * Context.NoV - RefractionProjectionTerm, 0.001, 1.0); // Due to CalcThinTransmission and Vis_SmithJointAniso, we need to make sure
	RefractedContext.NoL = clamp(Eta * Context.NoL - RefractionProjectionTerm, 0.001, 1.0); // those values are not 0s to avoid NaNs.
	RefractedContext.VoH = saturate(Eta * Context.VoH - RefractionBlendFactor);
	RefractedContext.VoL = 2.0 * RefractedContext.VoH * RefractedContext.VoH - 1.0;
	RefractedContext.NoH = Context.NoH;
	return RefractedContext;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting ClearCoatBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	const float ClearCoat			= GBuffer.CustomData.x;
	const float ClearCoatRoughness	= max(GBuffer.CustomData.y, 0.02f);

	FDirectLighting Lighting = {
		float3(0.0, 0.0, 0.0),
		float3(0.0, 0.0, 0.0),
		float3(0.0, 0.0, 0.0)
	};
	
	BxDFContext Context;
	half3 Nspec = N;

	if (CLEAR_COAT_BOTTOM_NORMAL)
	{
		Nspec = GBuffer.WorldNormal;
	}

#if SUPPORTS_ANISOTROPIC_MATERIALS
	bool bHasAnisotropy = HasAnisotropy(GBuffer.SelectiveOutputMask);
#else
	bool bHasAnisotropy = false;
#endif

	half3 X = 0;
	half3 Y = 0;

	//////////////////////////////
	/// Top Layer
	//////////////////////////////

	// No anisotropy for the top layer
	Init(Context, Nspec, V, AreaLight.SpecularL);
	
	// Modify SphereSinAlpha, knowing that it was previously manipulated by roughness of the under coat
	// Note: the operation is not invertible for GBuffer.Roughness = 1.0, so roughness is clamped to 254.0/255.0
	float SphereSinAlpha = AreaLight.SphereSinAlpha;
	float RoughnessCompensation = 1 - Pow2(GBuffer.Roughness);
	half Alpha = ClearCoatRoughness * ClearCoatRoughness;
	RoughnessCompensation = RoughnessCompensation > 0.0 ? (1 - Alpha) / RoughnessCompensation : 0.0;
	AreaLight.SphereSinAlpha = saturate(AreaLight.SphereSinAlpha * RoughnessCompensation);

	SphereMaxNoH(Context, AreaLight.SphereSinAlpha, CLEAR_COAT_BOTTOM_NORMAL == 0);
	Context.NoV = saturate(abs(Context.NoV) + 1e-5);
	const bool bIsRect = IsRectLight(AreaLight);
	Context.VoH = bIsRect ? Context.NoV : Context.VoH;

	// Hard-coded Fresnel evaluation with IOR = 1.5 (for polyurethane cited by Disney BRDF)
	float F0 = 0.04;
	float Fc = Pow5(1 - Context.VoH);
	float F = Fc + (1 - Fc) * F0;

	FBxDFEnergyTerms EnergyTermsCoat   = ComputeGGXSpecEnergyTerms(ClearCoatRoughness, Context.NoV, F0);

	if (bIsRect)
	{
		Lighting.Specular = ClearCoat * RectGGXApproxLTC(ClearCoatRoughness, F0, Nspec, V, AreaLight.Rect, AreaLight.Texture);
	}
	else
	{
		// Generalized microfacet specular
		float a2 = Pow2(Alpha);
		float ClearCoatEnergy = EnergyNormalization(a2, Context.VoH, AreaLight);
		half Vis = Vis_SmithJointApprox(a2, Context.NoV, AreaLight.NoL);
		float D = D_GGX(a2, Context.NoH) * ClearCoatEnergy;
		half Fr1 = (D * Vis) * F;
		Lighting.Specular = ClearCoat * AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL * Fr1);
	}
	Lighting.Specular *= ComputeEnergyConservation(EnergyTermsCoat);

	// Restore previously changed SphereSinAlpha for the top layer. 
	// Alpha needs to also be restored to the bottom layer roughness.
	AreaLight.SphereSinAlpha = SphereSinAlpha;
	Alpha = Pow2(GBuffer.Roughness);

	// Incoming and exiting Fresnel terms are identical to incoming Fresnel term (VoH == HoL)
	// float FresnelCoeff = (1.0 - F1) * (1.0 - F2);
	#if USE_ENERGY_CONSERVATION
	float3 FresnelCoeff = ComputeEnergyPreservation(EnergyTermsCoat); // 1.0 - F;
	#else
	// Preserve old behavior when energy conservation is disabled
	half FresnelCoeff = 1.0 - F;
	#endif
	FresnelCoeff *= FresnelCoeff;

	//////////////////////////////
	/// Bottom Layer
	//////////////////////////////

	if (CLEAR_COAT_BOTTOM_NORMAL)
	{
		BxDFContext TempContext;

		BRANCH
		if (bHasAnisotropy)
		{
			Init(TempContext, N, X, Y, V, AreaLight.SpecularL);
		}
		else
		{
			Init(TempContext, Nspec, V, AreaLight.SpecularL);
		}

		// If bottom-normal, update normal-based dot products:
		float3 H = normalize(V + AreaLight.SpecularL);
		Context.NoH = saturate(dot(N, H));
		Context.NoV = saturate(dot(N, V));
		Context.NoL = saturate(dot(N, AreaLight.SpecularL));
		Context.VoL = saturate(dot(V, AreaLight.SpecularL));
		Context.VoH = saturate(dot(V, H));

		Context.XoV = TempContext.XoV;
		Context.XoL = TempContext.XoL;
		Context.XoH = TempContext.XoH;
		Context.YoV = TempContext.YoV;
		Context.YoL = TempContext.YoL;
		Context.YoH = TempContext.YoH;

		if (!bHasAnisotropy)
		{
			bool bNewtonIteration = true;
			SphereMaxNoH(Context, AreaLight.SphereSinAlpha, bNewtonIteration);
		}

		Context.NoV = saturate(abs(Context.NoV) + 1e-5);
	}

	// Propagate refraction through dot-products rather than the original vectors:
	// Reference:
	//   float Eta = 1.0 / 1.5;
	//   float3 H = normalize(V + L);
	//   float3 V2 = Refract(V, H, Eta);
	//   float3 L2 = reflect(V2, H);
	//   V2 = -V2;
	//   BxDFContext BottomContext;
	//   Init(BottomContext, N, X, Y, V2, L2);
	if (bHasAnisotropy)
	{
		// Prepare the anisotropic Context to refract.
		X = GBuffer.WorldTangent;
		Y = normalize(cross(N, X));
		Init(Context, Nspec, X, Y, V, AreaLight.SpecularL);
	}
	BxDFContext BottomContext = RefractClearCoatContext(Context);
	BottomContext.VoH = bIsRect ? BottomContext.NoV : BottomContext.VoH;

	FBxDFEnergyTerms EnergyTermsBottom = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, BottomContext.NoV, GBuffer.SpecularColor);

	// Absorption
	float3 Transmission = SimpleClearCoatTransmittance(BottomContext.NoL, BottomContext.NoV, GBuffer.Metallic, GBuffer.BaseColor);

	// Default Lit
	half3 DefaultDiffuse = (AreaLight.Falloff * AreaLight.NoL) * AreaLight.FalloffColor * Diffuse_Lambert(GBuffer.DiffuseColor) * ComputeEnergyPreservation(EnergyTermsBottom);
	half3 RefractedDiffuse = FresnelCoeff * Transmission * DefaultDiffuse;
	Lighting.Diffuse = lerp(DefaultDiffuse, RefractedDiffuse, ClearCoat);

	if (!bHasAnisotropy && bIsRect)
	{
		// Note: V is used instead of V2 because LTC integration is not tuned to handle refraction direction
		float3 DefaultSpecular = RectGGXApproxLTC(GBuffer.Roughness, GBuffer.SpecularColor, N, V, AreaLight.Rect, AreaLight.Texture);
		float3 RefractedSpecular = FresnelCoeff * Transmission * DefaultSpecular;
		Lighting.Specular += lerp(DefaultSpecular, RefractedSpecular, ClearCoat);
	}
	else
	{
		float a2 = Pow4(GBuffer.Roughness);
		half D2 = 0;
		half Vis2 = 0;

		// The energy factor for correcting area light shape needs to be computed "before" the actual D_GGX term, 
		// as calling EnergyNormalization() modify the "a2" value.
		float Energy = 1;
		BRANCH
		if (!bHasAnisotropy)
		{
			Energy = EnergyNormalization(a2, Context.VoH, AreaLight);
		}

		BRANCH
		if (bHasAnisotropy)
		{
			float ax = 0;
			float ay = 0;
			GetAnisotropicRoughness(Alpha, GBuffer.Anisotropy, ax, ay); 

			D2 = D_GGXaniso(ax, ay, Context.NoH, Context.XoH, Context.YoH);
			Vis2 = Vis_SmithJointAniso(ax, ay, BottomContext.NoV, BottomContext.NoL, BottomContext.XoV, BottomContext.XoL, BottomContext.YoV, BottomContext.YoL);
		}
		else
		{
			// NoL is chosen to provide better parity with DefaultLit when ClearCoat=0
			Vis2 = Vis_SmithJointApprox(a2, BottomContext.NoV, AreaLight.NoL);
			D2 = D_GGX(a2, BottomContext.NoH);
		}
		half3 F_Bot = F_Schlick(GBuffer.SpecularColor, BottomContext.VoH);
		half3 F_DefaultLit = F_Schlick(GBuffer.SpecularColor, Context.VoH);
		

		// Note: reusing D and V from refracted context to save computation when ClearCoat < 1
		float3 CommonSpecular = (Energy * AreaLight.Falloff * AreaLight.NoL * D2 * Vis2) * AreaLight.FalloffColor;
		float3 DefaultSpecular = F_DefaultLit;
		float3 RefractedSpecular = FresnelCoeff * Transmission * F_Bot;
		Lighting.Specular += CommonSpecular * lerp(DefaultSpecular, RefractedSpecular, ClearCoat);
	}

	return Lighting;
}

// HG phase function approximated
float ApproximateHG(float cosJ, float g)
{
	float g2 = g * g;
	float gcos2 = 1.0f - (g * cosJ);
	gcos2 *= gcos2;

	const float ISO_PHASE_FUNC_Normalized = 0.5;

	return (ISO_PHASE_FUNC_Normalized * (1.0f - g2) / max( 1e-5, gcos2));
}

void GetProfileDualSpecular(uint SubsurfaceProfileInt, half Roughness, half Opacity, out half LobeRoughness0, out half LobeRoughness1, out half LobeMix)
{
#if !FORWARD_SHADING
	GetSubsurfaceProfileDualSpecular(SubsurfaceProfileInt, Roughness, Opacity, LobeRoughness0, LobeRoughness1, LobeMix);
#else
	// Disable dual lobe, as subsurface profile doesn't work with forward.
	LobeRoughness0 = Roughness;
	LobeRoughness1 = Roughness;
	LobeMix = 0.f;
#endif
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting SubsurfaceProfileBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	BxDFContext Context;
	Init( Context, N, V, AreaLight.SpecularL );
	SphereMaxNoH( Context, AreaLight.SphereSinAlpha, true );
	Context.NoV = saturate( abs( Context.NoV ) + 1e-5 );

	uint SubsurfaceProfileId = ExtractSubsurfaceProfileInt(GBuffer);
	half Opacity = GBuffer.CustomData.a;
	half Roughness = GBuffer.Roughness;

	half Lobe0Roughness = 0;
	half Lobe1Roughness = 0;
	half LobeMix = 0;

	GetProfileDualSpecular(SubsurfaceProfileId, Roughness, Opacity, Lobe0Roughness, Lobe1Roughness, LobeMix);
	half AverageRoughness = lerp(Lobe0Roughness, Lobe1Roughness, LobeMix);

	// Take the average roughness instead of compute a separate energy term for each roughness
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(AverageRoughness, Context.NoV, GBuffer.SpecularColor);

	FDirectLighting Lighting;
#if SHADING_PATH_MOBILE
	half Curvature = GBuffer.Curvature;
	half UnClampedNoL = dot(GBuffer.WorldNormal, AreaLight.DiffuseL);
	half ShadowFactor = 1.0f - sqrt(Shadow.SurfaceShadow);
	// Rotate the world normal based on the shadow value, it's just a experimental value 
	half UnClampedRotatedNoL = max(UnClampedNoL - max(2.0f * UnClampedNoL, 0.4f) * ShadowFactor, -1.0f);
	half4 BurleyDiffuse = GetSSProfilePreIntegratedValue(SubsurfaceProfileId, UnClampedRotatedNoL, Curvature);
	// asset specific color
	half3 Tint = GetSubsurfaceProfileTexture(SSSS_TINT_SCALE_OFFSET, SubsurfaceProfileId).rgb;
	
	// Needs to apply shadow at here, since the preintegrated value has already counted it, and we need to skip applying shadow outside.
	Lighting.Diffuse = (AreaLight.Falloff * AreaLight.NoL) * lerp(AreaLight.FalloffColor * Shadow.SurfaceShadow, BurleyDiffuse.rgb, Tint) * Diffuse_Lambert(GBuffer.DiffuseColor);
#else

#if MATERIAL_ROUGHDIFFUSE
	// Use Chan's diffuse model. It reduces flatness look and has better match, e.g., at the edge of human face skin when compared to GT.
	const float3 DiffuseReflection = Diffuse_GGX_Rough(GBuffer.DiffuseColor, GBuffer.Roughness, Context.NoV, AreaLight.NoL, Context.VoH, Context.NoH, GetAreaLightDiffuseMicroReflWeight(AreaLight));
#else
	const float3 DiffuseReflection = Diffuse_Burley(GBuffer.DiffuseColor, GBuffer.Roughness, Context.NoV, AreaLight.NoL, Context.VoH);
#endif

	Lighting.Diffuse  = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * DiffuseReflection;
#endif

	if (IsRectLight(AreaLight))
	{
		float3 Lobe0Specular = RectGGXApproxLTC(Lobe0Roughness, GBuffer.SpecularColor, N, V, AreaLight.Rect, AreaLight.Texture);
		float3 Lobe1Specular = RectGGXApproxLTC(Lobe1Roughness, GBuffer.SpecularColor, N, V, AreaLight.Rect, AreaLight.Texture);
		Lighting.Specular = lerp(Lobe0Specular, Lobe1Specular, LobeMix);
	}
	else
	{
		Lighting.Specular = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * DualSpecularGGX(AverageRoughness, Lobe0Roughness, Lobe1Roughness, LobeMix, GBuffer.SpecularColor, Context, AreaLight);
	}

	Lighting.Diffuse  *= ComputeEnergyPreservation(EnergyTerms);
	Lighting.Specular *= ComputeEnergyConservation(EnergyTerms);

#if USE_TRANSMISSION

	const uint ProfileId = ExtractSubsurfaceProfileInt(GBuffer);
	FTransmissionProfileParams TransmissionParams = GetTransmissionProfileParams(ProfileId);

	float ShadowOpticalDepth= DecodeOpticalDepthFromShadowMask(Shadow.TransmittanceOrOpticalThickness);
	float3 Profile = GetTransmissionProfile(ProfileId, ShadowOpticalDepth).rgb; // GetTransmissionProfile Thickness input is optical depth?

	float3 RefracV = refract(V, -N, TransmissionParams.OneOverIOR);
	float PhaseFunction = ApproximateHG( dot(-AreaLight.DiffuseL, RefracV), TransmissionParams.ScatteringDistribution );
	Lighting.Transmission = AreaLight.FalloffColor * Profile * (AreaLight.Falloff * PhaseFunction); // TODO: This probably should also include cosine term (NoL)

#else // USE_TRANSMISSION

	Lighting.Transmission = 0;

#endif // USE_TRANSMISSION

	return Lighting;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting ClothBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	const float3 FuzzColor	= ExtractSubsurfaceColor(GBuffer);
	const float  Cloth		= saturate(GBuffer.CustomData.a);

	BxDFContext Context;
	Init( Context, N, V, AreaLight.SpecularL );
	SphereMaxNoH( Context, AreaLight.SphereSinAlpha, true );
	Context.NoV = saturate( abs( Context.NoV ) + 1e-5 );

	float3 Spec1;
	if(IsRectLight(AreaLight))
		Spec1 = RectGGXApproxLTC( GBuffer.Roughness, GBuffer.SpecularColor, N, V, AreaLight.Rect, AreaLight.Texture );
	else
		Spec1 = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * SpecularGGX( GBuffer.Roughness, GBuffer.SpecularColor, Context, AreaLight );

	const FBxDFEnergyTerms EnergyTerms1 = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, Context.NoV, GBuffer.SpecularColor);
	Spec1 *= ComputeEnergyConservation(EnergyTerms1);

	// Cloth - Asperity Scattering - Inverse Beckmann Layer
	float D2 = D_InvGGX( Pow4( GBuffer.Roughness ), Context.NoH );
	float Vis2 = Vis_Cloth( Context.NoV, AreaLight.NoL );
#if SHADING_PATH_MOBILE
	Vis2 = saturate(Vis2);
#endif
	float3 F2 = F_Schlick( FuzzColor, Context.VoH );
	float3 Spec2 = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * (D2 * Vis2) * F2;
	
	const FBxDFEnergyTermsA EnergyTerms2 = ComputeClothEnergyTermsA(GBuffer.Roughness, Context.NoV);
	Spec2 *= ComputeEnergyConservation(EnergyTerms2);

	FDirectLighting Lighting;
	Lighting.Diffuse  = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * Diffuse_Lambert( GBuffer.DiffuseColor ); 
	Lighting.Specular = lerp( Spec1, Spec2, Cloth );
	Lighting.Transmission = 0;

	Lighting.Diffuse *= lerp(ComputeEnergyPreservation(EnergyTerms1), ComputeEnergyPreservation(EnergyTerms2), Cloth);

	return Lighting;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting SubsurfaceBxDF(FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	FDirectLighting Lighting = DefaultLitBxDF( GBuffer, N, V, AreaLight, Shadow);

	half3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
	half Opacity = GBuffer.CustomData.a;

	// to get an effect when you see through the material
	// hard coded pow constant
	half InScatter = pow(saturate(dot(AreaLight.DiffuseL, -V)), 12) * lerp(3, .1f, Opacity);

	// Wrap around lighting, 
	// * /(PI*2) to be energy consistent (hack do get some view dependnt and light dependent effect)
	// * Opacity of 0 gives no normal dependent lighting, Opacity of 1 gives strong normal contribution	
	// * Simplified version (with w=.5,  n=1.5): 
	//     half WrappedDiffuse = 2 * pow(saturate((dot(N, L) + w) / (1.0f + w)), n) * (n + 1) / (2 * (1 + w));
	//     NormalContribution = WrappedDiffuse * Opacity + 1 - Opacity;
	const half WrappedDiffuse = pow(saturate(dot(N, AreaLight.DiffuseL) * (1.f / 1.5f) + (0.5f / 1.5f)), 1.5f) * (2.5f / 1.5f);
	const half NormalContribution = lerp(1.f, WrappedDiffuse, Opacity);
	const half BackScatter = GBuffer.GBufferAO * NormalContribution / (PI * 2);

	// Transmission
	// * Emulate Beer-Lambert absorption by retrieving extinction coefficient from SubSurfaceColor. Subsurface is interpreted as a 'transmittance color' 
	//   at a certain 'normalized' distance (SubSurfaceColorAsTransmittanceAtDistanceInMeters). This is a coarse approximation for getting hue-shiting. 
	// * TransmittedColor is computed for the 1-normalized distance, and then transformed back-and-forth in HSV space to preserving the luminance value 
	//   of the original color, but getting hue shifting
	const half3 ExtinctionCoefficients = TransmittanceToExtinction(SubsurfaceColor, View.SubSurfaceColorAsTransmittanceAtDistanceInMeters);
	const half3 RawTransmittedColor = ExtinctionToTransmittance(ExtinctionCoefficients, 1.0f /*At 1 meters, as we use normalized units*/);
	const half3 TransmittedColor = HSV_2_LinearRGB(half3(LinearRGB_2_HSV(RawTransmittedColor).xy, LinearRGB_2_HSV(SubsurfaceColor).z));

	// lerp to never exceed 1 (energy conserving)
	const float SubsurfaceTransmittance = Shadow.TransmittanceOrOpticalThickness;
	Lighting.Transmission = AreaLight.FalloffColor * (AreaLight.Falloff * lerp(BackScatter, 1, InScatter)) * lerp(TransmittedColor, SubsurfaceColor, SubsurfaceTransmittance);

	return Lighting;
}

#if 0 // SGGX ref
float PDF_GGX( float a2, half3 N, half3 H )
{
	half NoH = saturate( dot(N,H) );
	return D_GGX( a2, NoH ) * NoH;
}

float3 SimpleSpec( FGBufferData GBuffer, half3 N, half3 V, half3 L )
{
	half3 H = normalize( L + V );

	half NoL = saturate( dot(N,L) );
	half NoV = saturate( dot(N,V) );
	half VoL = saturate( dot(V,L) );
	half NoH = saturate( dot(N,H) );
	half VoH = saturate( dot(V,H) );

	float a2 = Pow4( GBuffer.Roughness );

	// Generalized microfacet specular
	float D = D_GGX( a2, NoH );
	float Vis = Vis_SmithJoint( a2, max( NoV, 1e-5 ), NoL );
	float3 F = F_Schlick( GBuffer.SpecularColor, VoH );

	return (D * Vis) * F;
}

FDirectLighting ShadeSGGX( FGBufferData GBuffer, float Geo_a2, half3 V, half3 L )
{
	FDirectLighting Lighting = (FDirectLighting)0;

	half VoL = dot(V, L);
	half3 H = normalize( L + V );

	float Mat_a2 = Pow4( GBuffer.Roughness );

	const uint NumSamples = 64;

	LOOP
	for( uint i = 0; i < NumSamples; i++ )
	{
		RandomSequence RandSequence;
		RandomSequence_Initialize( RandSequence, View.StateFrameIndex, i );

		float4 Sample_VNDF = SGGX_DvisSample( RandomSequence_GenerateSample2D( RandSequence ), Geo_a2, V );
		float MoL_VNDF = saturate( dot( Sample_VNDF.xyz, L ) );

		{
			float4 Sample_CosL = CosineSampleHemisphere( RandomSequence_GenerateSample2D( RandSequence ), L );
			float MoL_CosL = saturate( dot( Sample_CosL.xyz, L ) );
			float Dvis_CosL = SGGX_Dvis( Geo_a2, V.z, dot( V, Sample_CosL.xyz ), Sample_CosL.z );

			Lighting.Diffuse += MoL_VNDF * MISWeightBalanced( Sample_VNDF.w, MoL_VNDF / PI );
			//Lighting.Diffuse += Dvis_CosL * MoL_CosL / Sample_CosL.w * MISWeightBalanced( Sample_CosL.w, Dvis_CosL );
			Lighting.Diffuse += Dvis_CosL * PI * MISWeightBalanced( Sample_CosL.w, Dvis_CosL );
		}

		{
			float4 Sample_Material = ImportanceSampleGGX( RandomSequence_GenerateSample2D( RandSequence ), Mat_a2 );
			Sample_Material.xyz = mul( Sample_Material.xyz, GetTangentBasis(H) );

			float Dvis_Material = SGGX_Dvis( Geo_a2, V.z, dot( V, Sample_Material.xyz ), Sample_Material.z );
			float MoL_Material = saturate( dot( Sample_Material.xyz, L ) );

			Lighting.Specular += SimpleSpec( GBuffer, Sample_VNDF.xyz, V, L ) * MoL_VNDF * MISWeightBalanced( Sample_VNDF.w, PDF_GGX( Mat_a2, Sample_VNDF.xyz, H ) );
			Lighting.Specular += SimpleSpec( GBuffer, Sample_Material.xyz, V, L ) * MoL_Material * Dvis_Material / Sample_Material.w * MISWeightBalanced( Sample_Material.w, Dvis_Material );
		}

		{
			// http://blog.stevemcauley.com/2011/12/03/energy-conserving-wrapped-diffuse/
			half Wrap = 0.5;
			half WrapNoL = saturate( ( -dot( Sample_VNDF.xyz, L ) + Wrap ) / Square( 1 + Wrap ) );

			// Scatter distribution
			float Scatter = D_GGX( 0.6*0.6, saturate( -VoL ) );

			Lighting.Transmission += WrapNoL * Scatter;
		}
	}

	Lighting.Diffuse		/= NumSamples;
	Lighting.Specular		/= NumSamples;
	Lighting.Transmission	/= NumSamples;

	Lighting.Diffuse		*= GBuffer.DiffuseColor / PI;
	Lighting.Transmission	*= ExtractSubsurfaceColor( GBuffer );

	return Lighting;
}
#endif

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting TwoSidedBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	half3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);

#if 0
	half NormalLength2 = length2( GBuffer.NormalDistribution );
	half RcpNormalLength = rsqrt( NormalLength2 );
	half NormalLength = NormalLength2 * RcpNormalLength;
	N = GBuffer.NormalDistribution * RcpNormalLength;

	half3 L = AreaLight.DiffuseL;
	half NoL = dot(N, L);
	half NoV = dot(N, V);
	half VoL = dot(V, L);
	half InvLenH = rsqrt( 2 + 2 * VoL );
	half NoH = ( NoL + NoV ) * InvLenH;
	half VoH = InvLenH + InvLenH * VoL;
	half3 H = ( L + V ) * InvLenH;

	float Geo_a = 0.5 - 0.5 * NormalLength;
	if( NoV < 0 )
		Geo_a = 1 - Geo_a;

	Geo_a = clamp( Geo_a, 0.01, 0.999 );

	if( Geo_a > 0.5 )
		Geo_a = rcp( 2 - 2 * Geo_a );
	else
		Geo_a = 2 * Geo_a;

	float Geo_a2 = Pow2( Geo_a );
	float Geo_Area = SGGX_SurfaceArea( Geo_a2 );

	float3x3 TangentToWorld = GetTangentBasis( N );

	float3 TangentV = mul( TangentToWorld, V );
	float3 TangentL = mul( TangentToWorld, L );

	float3 SphereScale = 1;
	SphereScale.xy = Geo_a;

	float3 SphereV = normalize( TangentV * SphereScale );
	float3 SphereL = normalize( TangentL * SphereScale );

	float SphereVoL = dot( SphereV, SphereL );

#if 0
	float vNoV = SGGX_ProjectedArea( Geo_a2, NoV ) * 2 / Geo_Area;
	float vNoL = SGGX_ProjectedArea( Geo_a2, NoL ) * 2 / Geo_Area;
#else
	float vNoV = 2 * TangentV.z / ( SphereV.z * Geo_Area );
	float vNoL = 2 * TangentL.z / ( SphereL.z * Geo_Area );
#endif

	vNoL *= SGGX_Diffuse( SphereVoL );
	vNoL *= lerp( 0.5 * PI, 2, saturate( Geo_a ) );

	FDirectLighting Lighting = (FDirectLighting)0;

	{
		Lighting.Diffuse = AreaLight.FalloffColor * (AreaLight.Falloff * vNoL) * Diffuse_Lambert( GBuffer.DiffuseColor );
	}

	{
		float Mat_a2 = Pow4( GBuffer.Roughness );
		float Spec_a2 = SGGX_Convolve( Geo_a2, Mat_a2 );

		float D = D_GGX( Spec_a2, NoH );

		float Vis_SmithV = vNoL * SGGX_ProjectedArea( Spec_a2, NoV );
		float Vis_SmithL = vNoV * SGGX_ProjectedArea( Spec_a2, NoL );

		//Vis_SmithV = lerp( vNoL * SGGX_ProjectedArea( Mat_a2, vNoV ), Vis_SmithV, VoH );
		Vis_SmithL = lerp( vNoV * SGGX_ProjectedArea( Mat_a2, vNoL ), Vis_SmithL, VoH );

		float Vis = 0.5 * rcp( Vis_SmithV + Vis_SmithL );

		float3 F = F_Schlick( GBuffer.SpecularColor, VoH );

		Lighting.Specular = AreaLight.FalloffColor * (AreaLight.Falloff * vNoL * D * Vis) * F;
	}

	{
		float InvNoL = -0.6 * SphereVoL;
		InvNoL *= TangentL.z / SphereL.z;
		InvNoL /= 0.25 * Geo_Area;

		// http://blog.stevemcauley.com/2011/12/03/energy-conserving-wrapped-diffuse/
		half Wrap = 0.5;
		half WrapNoL = saturate( ( InvNoL + Wrap ) / Square( 1 + Wrap ) );

		// Scatter distribution
		float Scatter = D_GGX( 0.6*0.6, saturate( -VoL ) );

		Lighting.Transmission = AreaLight.FalloffColor * (AreaLight.Falloff * WrapNoL * Scatter) * SubsurfaceColor;
	}
#else
	FDirectLighting Lighting = DefaultLitBxDF( GBuffer, N, V, AreaLight, Shadow );

	// http://blog.stevemcauley.com/2011/12/03/energy-conserving-wrapped-diffuse/
	half Wrap = 0.5;
	half WrapNoL = saturate( ( -dot(N, AreaLight.DiffuseL) + Wrap ) / Square( 1 + Wrap ) );

	// Scatter distribution
	half VoL = dot(V, AreaLight.DiffuseL);
	float Scatter = D_GGX( 0.6*0.6, saturate( -VoL ) );

	Lighting.Transmission = AreaLight.FalloffColor * (AreaLight.Falloff * WrapNoL * Scatter) * SubsurfaceColor;
#endif

	return Lighting;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FDirectLighting EyeBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
#if IRIS_NORMAL
	const float2 CausticNormalDelta	= float2( GBuffer.StoredMetallic, GBuffer.StoredSpecular )	* 2 - (256.0/255.0);
	const float2 IrisNormalDelta	= float2( GBuffer.CustomData.y, GBuffer.CustomData.z )		* 2 - (256.0/255.0);
	const float  IrisMask = 1.0f - GBuffer.CustomData.w;

	const float2 WorldNormalOct	= UnitVectorToOctahedron( GBuffer.WorldNormal );
	const float3 CausticNormal	= OctahedronToUnitVector( WorldNormalOct + CausticNormalDelta );
	const float3 IrisNormal		= OctahedronToUnitVector( WorldNormalOct + IrisNormalDelta );
#else
	const float3 IrisNormal		= OctahedronToUnitVector( GBuffer.CustomData.yz * 2 - 1 );
	const float  IrisDistance	= GBuffer.StoredMetallic;
	const float  IrisMask		= 1.0f - GBuffer.CustomData.w;

	// Blend in the negative intersection normal to create some concavity
	// Not great as it ties the concavity to the convexity of the cornea surface
	// No good justification for that. On the other hand, if we're just looking to
	// introduce some concavity, this does the job.
	const float3 CausticNormal = normalize(lerp(IrisNormal, -N, IrisMask*IrisDistance));
#endif

	BxDFContext Context;
	Init( Context, N, V, AreaLight.SpecularL );
	SphereMaxNoH( Context, AreaLight.SphereSinAlpha, false );
	Context.NoV = saturate( abs( Context.NoV ) + 1e-5 );
	const bool bIsRect = IsRectLight(AreaLight);
	Context.VoH = bIsRect ? Context.NoV : Context.VoH;
	
	// F_Schlick
	float F0 = GBuffer.Specular * 0.08;
	float Fc = Pow5( 1 - Context.VoH );
	float F = Fc + (1 - Fc) * F0;
	
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, Context.NoV, F0);

	FDirectLighting Lighting;

	if( bIsRect )
	{
		Lighting.Specular = RectGGXApproxLTC( GBuffer.Roughness, F0, N, V, AreaLight.Rect, AreaLight.Texture );
	}
	else
	{
		float a2 = Pow4( GBuffer.Roughness );
		float Energy = EnergyNormalization( a2, Context.VoH, AreaLight );
		// Generalized microfacet specular
		float Vis = Vis_SmithJointApprox(a2, Context.NoV, AreaLight.NoL);
		float D = D_GGX(a2, Context.NoH) * Energy;
		Lighting.Specular = AreaLight.FalloffColor * (AreaLight.Falloff * AreaLight.NoL) * D * Vis * F;
	}

	float IrisNoL = saturate( dot( IrisNormal, AreaLight.SpecularL ) );
	float Power = lerp( 12, 1, IrisNoL );
	float Caustic = 0.8 + 0.2 * ( Power + 1 ) * pow( saturate( dot( CausticNormal, AreaLight.SpecularL ) ), Power );
	float Iris = IrisNoL * Caustic;
	float Sclera = AreaLight.NoL;
	
	Lighting.Specular *= ComputeEnergyConservation(EnergyTerms);

#if USE_ENERGY_CONSERVATION
	const float3 EnergyPreservation = ComputeEnergyPreservation(EnergyTerms);
#else
	// Preserve old behavior when energy conservation is disabled
	const float EnergyPreservation = 1.0f - F;
#endif

	Lighting.Diffuse = 0;
	Lighting.Transmission = AreaLight.FalloffColor * ( AreaLight.Falloff * lerp( Sclera, Iris, IrisMask ) * EnergyPreservation ) * Diffuse_Lambert( GBuffer.DiffuseColor );
#if SHADING_PATH_MOBILE
	uint SubsurfaceProfileId = ExtractSubsurfaceProfileInt(GBuffer);
	half Curvature = GBuffer.Curvature;
	half ShadowFactor = 1.0f - sqrt(Shadow.SurfaceShadow);
	
	// Rotate the world normal based on the shadow value, it's just a experimental value 
	half UnClampedNoL = dot(GBuffer.WorldNormal, AreaLight.DiffuseL);
	half UnClampedRotatedNoL = max(UnClampedNoL - max(2.0f * UnClampedNoL, 0.4f) * ShadowFactor, -1.0f);
	half4 BurleyDiffuse = GetSSProfilePreIntegratedValue(SubsurfaceProfileId, UnClampedRotatedNoL, Curvature);

	// Rotate the world normal based on the shadow value, it's just a experimental value 
	half UnClampedIrisNoL = dot(IrisNormal, AreaLight.SpecularL);
	half UnClampedRotatedIrisNoL = max(UnClampedIrisNoL - max(2.0f * UnClampedIrisNoL, 0.4f) * ShadowFactor, -1.0f);
	half4 IrisBurleyDiffuse = GetSSProfilePreIntegratedValue(SubsurfaceProfileId, UnClampedRotatedIrisNoL * Caustic, Curvature);
	// asset specific color
	half3 Tint = GetSubsurfaceProfileTexture(SSSS_TINT_SCALE_OFFSET, SubsurfaceProfileId).rgb;

	// Needs to apply shadow at here, since the preintegrated value has already counted it, and we need to skip applying shadow outside.
	Lighting.Transmission = lerp(Lighting.Transmission * Shadow.SurfaceShadow, AreaLight.Falloff * lerp(BurleyDiffuse.rgb, IrisBurleyDiffuse.rgb, IrisMask) * Diffuse_Lambert(GBuffer.DiffuseColor), Tint);
#endif
	return Lighting;
}

FDirectLighting PreintegratedSkinBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	FDirectLighting Lighting = DefaultLitBxDF( GBuffer, N, V, AreaLight, Shadow );
	
	half3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
	half Opacity = GBuffer.CustomData.a;

	half3 PreintegratedBRDF = Texture2DSampleLevel(View.PreIntegratedBRDF, View.PreIntegratedBRDFSampler, float2(saturate(dot(N, AreaLight.DiffuseL) * .5 + .5), 1 - Opacity), 0).rgb;
	Lighting.Transmission = AreaLight.FalloffColor * AreaLight.Falloff * PreintegratedBRDF * SubsurfaceColor;

	return Lighting;
}

#if SUBSTRATE_ENABLED

#include "Substrate/SubstrateToonBSDF.ush"

// Mobile tech debt: adding support to legacy BxDF for now to not have to convert mobile to Substrate fully yet.
FDirectLighting ToonBxDF(FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms ShadowTerms)
{
	const bool bThisPassAppliesDiffuseRamp = true;
#if SUPPORTS_ANISOTROPIC_MATERIALS
	bool bHasAnisotropy = HasAnisotropy(GBuffer.SelectiveOutputMask);
#else
	bool bHasAnisotropy = false;
#endif
	const bool bIsRect = IsRectLight(AreaLight);
	//Context.VoH = bIsRect ? Context.NoV : Context.VoH;

	BxDFContext Context;
	float NoV, VoH, NoH;
	BRANCH
		if (bHasAnisotropy)
		{
			half3 X = GBuffer.WorldTangent;
			half3 Y = normalize(cross(N, X));
			Init(Context, N, X, Y, V, AreaLight.SpecularL);

			NoV = Context.NoV;
			VoH = Context.VoH;
			NoH = Context.NoH;
		}
		else
		{
			Init(Context, N, V, AreaLight.SpecularL);

			NoV = Context.NoV;
			VoH = Context.VoH;
			NoH = Context.NoH;

			SphereMaxNoH(Context, AreaLight.SphereSinAlpha, true);
		}

	float3 DiffuseColor = GBuffer.DiffuseColor;
	float3 SpecularColor = GBuffer.SpecularColor;
	float  Roughness = GBuffer.Roughness;
	float2 ToonPatternUVs = 0.0f;
	float ToonProfileId = 0.0f;
	UnpackToonCustomData(PackRGBA8(GBuffer.CustomData), ToonPatternUVs, ToonProfileId);

	float	Anisotropy = GBuffer.Anisotropy;
	float3	Emissive = 0;

	FSubstrateEvaluateToonResult ToonSample = SubstrateToonEvaluateCommon(
		DiffuseColor,
		SpecularColor,
		Roughness,
		ToonProfileId,
		ToonPatternUVs,
		Anisotropy,
		Emissive,
		Context,
		bHasAnisotropy,
		bIsRect,
		bThisPassAppliesDiffuseRamp,
		N, V, AreaLight, ShadowTerms.SurfaceShadow, ShadowTerms.TransmittanceOrOpticalThickness);

	FDirectLighting Lighting;
	Lighting.Diffuse = ToonSample.IntegratedDiffuseValue;
	Lighting.Specular = ToonSample.IntegratedSpecularValue;
	Lighting.Transmission = 0;
	return Lighting;
}
#endif // SUBSTRATE_ENABLED

// UE_DEPRECATED 5.7 - Deprecated by Substrate - Substrate uses SubstrateEvaluateBSDFCommon() instead
FDirectLighting IntegrateBxDF( FGBufferData GBuffer, half3 N, half3 V, FAreaLight AreaLight, FShadowTerms Shadow )
{
	switch( GBuffer.ShadingModelID )
	{
		case SHADINGMODELID_DEFAULT_LIT:
		case SHADINGMODELID_SINGLELAYERWATER:
		case SHADINGMODELID_THIN_TRANSLUCENT:
			return DefaultLitBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_SUBSURFACE:
			return SubsurfaceBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_PREINTEGRATED_SKIN:
			return PreintegratedSkinBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_CLEAR_COAT:
			return ClearCoatBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_SUBSURFACE_PROFILE:
			return SubsurfaceProfileBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_TWOSIDED_FOLIAGE:
			return TwoSidedBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_HAIR:
			return HairBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_CLOTH:
			return ClothBxDF( GBuffer, N, V, AreaLight, Shadow );
		case SHADINGMODELID_EYE:
			return EyeBxDF( GBuffer, N, V, AreaLight, Shadow );
	#if SUBSTRATE_ENABLED
		case SHADINGMODELID_SUBSTRATE_TOON:
			return ToonBxDF( GBuffer, N, V, AreaLight, Shadow );
	#endif // SUBSTRATE_ENABLED
		default:
			return (FDirectLighting)0;
	}
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate - Substrate uses SubstrateEvaluateBSDFCommon() instead
FDirectLighting EvaluateBxDF( FGBufferData GBuffer, half3 N, half3 V, half3 L, float NoL, FShadowTerms Shadow )
{
	FAreaLight AreaLight = InitAreaLight();
	AreaLight.DiffuseL = L;
	AreaLight.SpecularL = L;
	AreaLight.NoL = NoL;

	return IntegrateBxDF( GBuffer, N, V, AreaLight, Shadow );
}
