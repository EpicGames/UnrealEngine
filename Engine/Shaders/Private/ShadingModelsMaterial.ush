// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadingModelsMaterial.usf: Pixel shader function for computing a GBuffer from shading model.
=============================================================================*/

#include "SubsurfaceProfileCommon.ush"

// Optimization: if opacity is 0 then revert to default shading model 
#define SUBSURFACE_PROFILE_OPACITY_THRESHOLD 1

void SetGBufferForShadingModel(
	in out FGBufferData GBuffer, 
	in out FMaterialPixelParameters MaterialParameters,
	FPixelMaterialInputs PixelMaterialInputs,
	const float Opacity,
	const half3 BaseColor,
	const half  Metallic,
	const half  Specular,
	const float Roughness,
	const float Anisotropy,
	const float3 SubsurfaceColor,
	const float SubsurfaceProfile,
	const float Dither,
	const uint ShadingModel)
{
	GBuffer.WorldNormal = MaterialParameters.WorldNormal;
	GBuffer.WorldTangent = MaterialParameters.WorldTangent;
	GBuffer.BaseColor = BaseColor;
	GBuffer.Metallic = Metallic;
	#if SHADING_PATH_MOBILE
	GBuffer.Specular = Specular;
	#else
	// Dither value for avoiding quantization artifact (for now, only Specular is dithered based on artists' feedback)
	GBuffer.Specular = Dither8bits(Specular, Dither);
	#endif
	GBuffer.Roughness = Roughness;
	GBuffer.Anisotropy = Anisotropy;
	GBuffer.ShadingModelID = ShadingModel;

	// Calculate and set custom data for the shading models that need it
	
	// Dummy initial if statement to play nicely with the #ifdef logic
	if (false)
	{
	}
#if MATERIAL_SHADINGMODEL_SUBSURFACE
	else if (ShadingModel == SHADINGMODELID_SUBSURFACE)
	{
		GBuffer.CustomData.rgb = EncodeSubsurfaceColor(SubsurfaceColor);
		GBuffer.CustomData.a = Opacity;
	}
#endif
#if MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
	else if (ShadingModel == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		GBuffer.CustomData.rgb = EncodeSubsurfaceColor(SubsurfaceColor);
		GBuffer.CustomData.a = Opacity;
	}
#endif
#if MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE
	else if (ShadingModel == SHADINGMODELID_SUBSURFACE_PROFILE)
	{
		// Optimization: if opacity is 0 then revert to default shading model 
	#if SUBSURFACE_PROFILE_OPACITY_THRESHOLD
		if (Opacity > SSSS_OPACITY_THRESHOLD_EPS)
	#endif
		{
			GBuffer.CustomData.rgb = EncodeSubsurfaceProfile(SubsurfaceProfile);
			GBuffer.CustomData.a = Opacity;

		#if SHADING_PATH_MOBILE
			#if MATERIAL_SUBSURFACE_PROFILE_USE_CURVATURE
				GBuffer.Curvature = GetMaterialCustomData0(PixelMaterialInputs);
			#else
				GBuffer.Curvature = CalculateCurvature(GBuffer.WorldNormal, MaterialParameters.WorldPosition_CamRelative);
			#endif

			GBuffer.Curvature = clamp(GBuffer.Curvature, 0.001f, 1.0f);
		#endif
		}
	#if SUBSURFACE_PROFILE_OPACITY_THRESHOLD
		else
		{
			GBuffer.ShadingModelID = SHADINGMODELID_DEFAULT_LIT;
			GBuffer.CustomData = 0;
		}
	#endif
	}
#endif
#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	else if (ShadingModel == SHADINGMODELID_CLEAR_COAT)
	{
		float ClearCoat				= saturate( GetMaterialCustomData0(PixelMaterialInputs) );
		float ClearCoatRoughness	= saturate( GetMaterialCustomData1(PixelMaterialInputs) );
		GBuffer.CustomData.x = ClearCoat;
		GBuffer.CustomData.y = ClearCoatRoughness;

		// Clamp roughness to guarantee functional inverse when computing SphereSinAlpha for multiple layers
		GBuffer.Roughness = clamp(GBuffer.Roughness, 0.0, 254.0 / 255.0);

	#if CLEAR_COAT_BOTTOM_NORMAL
		{
			float2 oct2 = UnitVectorToOctahedron(GBuffer.WorldNormal);

			#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
				#if MATERIAL_TANGENTSPACENORMAL
					float3 tempnormal = normalize(TransformTangentVectorToWorld( MaterialParameters.TangentToWorld, ClearCoatBottomNormal0(MaterialParameters) ));
				#else
					float3 tempnormal =  ClearCoatBottomNormal0(MaterialParameters);
				#endif

				float2 oct1 = UnitVectorToOctahedron(tempnormal);
				float2 oct3 = ( (oct1 - oct2) *  0.25 ) + (128.0/255.0);
				GBuffer.CustomData.a = oct3.x;
				GBuffer.CustomData.z = oct3.y;
			#else
				GBuffer.CustomData.a = 128.0/255.0;
				GBuffer.CustomData.z = 128.0/255.0;
			#endif
		}
	#endif
	}
#endif
#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	else if (ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		GBuffer.CustomData.rgb = EncodeSubsurfaceColor(SubsurfaceColor);
		GBuffer.CustomData.a = Opacity;
	}
#endif
#if MATERIAL_SHADINGMODEL_HAIR
	else if (ShadingModel == SHADINGMODELID_HAIR)
	{
		GBuffer.CustomData.xy = UnitVectorToOctahedron( MaterialParameters.WorldNormal ) * 0.5 + 0.5;
		GBuffer.CustomData.z = saturate( GetMaterialCustomData0(PixelMaterialInputs) );	// Backlit
	}
#endif
#if MATERIAL_SHADINGMODEL_CLOTH
	else if (ShadingModel == SHADINGMODELID_CLOTH)
	{
		GBuffer.CustomData.rgb = EncodeSubsurfaceColor(SubsurfaceColor);
		GBuffer.CustomData.a = saturate( GetMaterialCustomData0(PixelMaterialInputs) );	// Cloth
		GBuffer.IndirectIrradiance *= 1 - GBuffer.CustomData.a;
	}
#endif
#if MATERIAL_SHADINGMODEL_EYE
	else if (ShadingModel == SHADINGMODELID_EYE)
	{
		const float IrisMask     = saturate(GetMaterialCustomData0(PixelMaterialInputs));
		const float IrisDistance = saturate(GetMaterialCustomData1(PixelMaterialInputs));

		GBuffer.CustomData.x = EncodeSubsurfaceProfile(SubsurfaceProfile).x;
		GBuffer.CustomData.w = 1.0f - IrisMask;	// Opacity

	#if IRIS_NORMAL
		float2 WorldNormalOct = UnitVectorToOctahedron( GBuffer.WorldNormal );

		// CausticNormal stored as octahedron
		#if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
			// Blend in the negative intersection normal to create some concavity
			// Not great as it ties the concavity to the convexity of the cornea surface
			// No good justification for that. On the other hand, if we're just looking to
			// introduce some concavity, this does the job.
			float3 PlaneNormal = normalize( GetTangentOutput0(MaterialParameters) );
			float3 CausticNormal = normalize( lerp( PlaneNormal, -GBuffer.WorldNormal, IrisMask*IrisDistance ) );
			float2 CausticNormalOct  = UnitVectorToOctahedron( CausticNormal );
			float2 CausticNormalDelta = ( CausticNormalOct - WorldNormalOct ) * 0.5 + (128.0/255.0);
			GBuffer.Metallic = CausticNormalDelta.x;
			GBuffer.Specular = CausticNormalDelta.y;
		#else
			float3 PlaneNormal = GBuffer.WorldNormal;
			GBuffer.Metallic = 128.0/255.0;
			GBuffer.Specular = 128.0/255.0;
		#endif

		// IrisNormal CustomData.yz
		#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
			float3 IrisNormal = normalize( ClearCoatBottomNormal0(MaterialParameters) );
			#if MATERIAL_TANGENTSPACENORMAL
				IrisNormal = normalize( TransformTangentVectorToWorld( MaterialParameters.TangentToWorld, IrisNormal ) );
			#endif
		#else
			float3 IrisNormal = PlaneNormal;
		#endif

		float2 IrisNormalOct  = UnitVectorToOctahedron( IrisNormal );
		float2 IrisNormalDelta = ( IrisNormalOct - WorldNormalOct ) * 0.5 + (128.0/255.0);
		GBuffer.CustomData.yz = IrisNormalDelta;
	#else
		GBuffer.Metallic = IrisDistance;

		#if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
			float3 Tangent = GetTangentOutput0(MaterialParameters);
			GBuffer.CustomData.yz = UnitVectorToOctahedron( normalize(Tangent) ) * 0.5 + 0.5;
		#endif
	#endif

	#if SHADING_PATH_MOBILE
		#if MATERIAL_SHADINGMODEL_EYE_USE_CURVATURE
			GBuffer.Curvature = Metallic;
		#else
			GBuffer.Curvature = CalculateCurvature(GBuffer.WorldNormal, MaterialParameters.WorldPosition_CamRelative);
		#endif

		GBuffer.Curvature = clamp(GBuffer.Curvature, 0.001f, 1.0f);
	#endif
	}
#endif
#if MATERIAL_SHADINGMODEL_TOON
	else if (ShadingModel == SHADINGMODELID_SUBSTRATE_TOON)
	{
		// GBuffer.CustomData is used
	}
#endif
}