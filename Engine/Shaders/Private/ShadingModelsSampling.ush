// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "ShadingCommon.ush"
//#include "HairStrands/HairBsdfSample.ush"
#include "ShadingModels.ush"
#include "MonteCarlo.ush"
#include "DeferredShadingCommon.ush"

#ifndef USE_HAIR_COMPLEX_TRANSMITTANCE
#define USE_HAIR_COMPLEX_TRANSMITTANCE 0
#endif
#if USE_HAIR_COMPLEX_TRANSMITTANCE
#include "HairStrands/HairStrandsCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceCommon.ush"
#include "HairStrands/HairStrandsDeepTransmittanceDualScattering.ush"
#endif

// Diffuse therm
#define SHADING_TERM_DIFFUSE 0x01

// Specular therm, need to apply EnvBRDF() before apply to scene color.
#define SHADING_TERM_SPECULAR 0x02

// Hair's specular therm
// TODO: should be same as SHADING_TERM_SPECULAR?
#define SHADING_TERM_HAIR_R 0x04

// Hair's double transmitance
#define SHADING_TERM_HAIR_TT 0x08

// Hair's inner reflection bound.
#define SHADING_TERM_HAIR_TRT 0x10


struct FBxDFSample
{
	// World space direction of the sample to do.
	float3 L;

	// PDF or the ray.
	float PDF;

	// Weight to apply to the sample's radiance.
	// EvaluateBxDF(L) / PDF
	float3 Weight;

	// Term that has been selected.
	uint Term;
};

FBxDFSample SampleDiffuseBxDF(float3 N, float4 E)
{
	float2 DiskE = UniformSampleDiskConcentric(E.xy);
	float3x3 TangentBasis = GetTangentBasis(N);
	FBxDFSample BxDFSample = (FBxDFSample)0;
	const float TangentZ = sqrt(1 - length2(DiskE));

	BxDFSample.L = mul(float3(DiskE, TangentZ), TangentBasis);
	BxDFSample.PDF = TangentZ * rcp(PI);
	BxDFSample.Weight = 1.0;
	BxDFSample.Term = SHADING_TERM_DIFFUSE;
	return BxDFSample;
}

FBxDFSample SampleDefaultLitBxDF(uint TermMask, float3 WorldNormal, float3x3 TangentBasis, float Anisotropy, float Roughness, float3 V, float4 E)
{
	TermMask &= SHADING_TERM_DIFFUSE | SHADING_TERM_SPECULAR;

	float2 DiskE = UniformSampleDiskConcentric(E.xy);
	float3 N = WorldNormal;

	// TODO: stocastically choose SHADING_TERM_D or SHADING_TERM_DIFFUSE if TermMask == (SHADING_TERM_D | SHADING_TERM_SPECULAR)

	FBxDFSample BxDFSample = (FBxDFSample)0;
	if (TermMask == SHADING_TERM_DIFFUSE)
	{
		float TangentZ = sqrt(1 - length2(DiskE));

		BxDFSample.L = mul(float3(DiskE, TangentZ), TangentBasis);
		BxDFSample.PDF = TangentZ * rcp(PI);
		BxDFSample.Weight = 1.0;
		BxDFSample.Term = SHADING_TERM_DIFFUSE;
	}
	else if (TermMask == SHADING_TERM_SPECULAR)
	{
		float2 Alpha = Pow2(Roughness).xx;

		if (Anisotropy != 0)
		{
			GetAnisotropicRoughness(Alpha.x, Anisotropy, Alpha.x, Alpha.y);
		}

		float3 TangentV = mul(TangentBasis, V);

#if 1
		// PDF = G_SmithV * VoH * D / NoV / (4 * VoH)
		// PDF = G_SmithV * D / (4 * NoV);
		float4 TangentH = ImportanceSampleVisibleGGX(E.xy, Alpha, TangentV);
#else
		// PDF = D * NoH / (4 * VoH),
		float4 TangentH = ImportanceSampleGGX(E.xy, Alpha * Alpha);
#endif

		float HPDF = TangentH.w;

		float3 H = mul(TangentH.xyz, TangentBasis);

		float VoH = saturate(dot(V, H));

		BxDFSample.L = 2 * dot(V, H) * H - V;
		BxDFSample.PDF = RayPDFToReflectionRayPDF(VoH, HPDF);

		// float D = D_GGX(a2, NoH);
		// float G_SmithL = 2 * NoL / (NoL + sqrt(NoL * (NoL - NoL * a2) + a2));
		// float Vis = Vis_Smith(a2, NoV, NoL);
		// float3 F = F_Schlick(GBuffer.SpecularColor, VoH);

		// Correct integration applies DGF below but for speed we apply EnvBRDF later when compositing
		// BxDFSample.Weight = F * ( NoL * Vis * (4 * VoH / NoH) ); // for ImportanceSampleGGX
		// BxDFSample.Weight = F * G_SmithL; // for ImportanceSampleVisibleGGX
		BxDFSample.Weight = 1.0;

		BxDFSample.Term = SHADING_TERM_SPECULAR;
	}

	return BxDFSample;
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FBxDFSample SampleDefaultLitBxDF(uint TermMask, FGBufferData GBuffer, float3 V, float4 E) 
{ 
	float3x3 TangentBasis;

	if (GBuffer.Anisotropy != 0) 
	{ 
		TangentBasis[0] = GBuffer.WorldTangent;
		TangentBasis[1] = cross(GBuffer.WorldNormal, GBuffer.WorldTangent);
		TangentBasis[2] = GBuffer.WorldNormal;
	} 
	else 
	{ 
		TangentBasis = GetTangentBasis(GBuffer.WorldNormal);
	}
	return SampleDefaultLitBxDF(TermMask, GBuffer.WorldNormal, TangentBasis, GBuffer.Anisotropy, GBuffer.Roughness, V, E); 
}

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FBxDFSample SampleHairBxDF(uint TermMask, FGBufferData GBuffer, float3 V, float4 E)
#if 1
{
	float4 L = UniformSampleSphere(E.xy);

	FDirectLighting Lighting;
	const float OpaqueVisibility = 1;	// No occlusion...
	const float Area = 0;				// This is used for faking area light sources by increasing the roughness of the surface. Disabled = 0.
	const float Backlit = 1; 			// This is used for suppressing the R & TT terms when when the lighting direction comes from behind. Disabled = 1.
	Lighting.Diffuse = 1; 				// Integrate over the entire sphere, and will reweight this with HairShading() during the composition. This is not correct, but give plausible result.
	Lighting.Transmission = 0;
	Lighting.Specular = 0;

	FBxDFSample BxDFSample;
	BxDFSample.L = L.xyz;
	BxDFSample.PDF = L.w;
	BxDFSample.Weight = 1;
	BxDFSample.Term = TermMask;

	return BxDFSample;
}
#else
{
	TermMask &= SHADING_TERM_HAIR_R | SHADING_TERM_HAIR_TT | SHADING_TERM_HAIR_TRT;

	float Backlit = GBuffer.CustomData.z;

	float3 T = GBuffer.WorldNormal;

	// TODO: should have a unique term bit mask instead?
	uint HairComponents = 0;
	HairComponents |= (TermMask & SHADING_TERM_HAIR_R) ? HAIR_COMPONENT_R : 0;
	HairComponents |= (TermMask & SHADING_TERM_HAIR_TT) ? HAIR_COMPONENT_TT : 0;
	HairComponents |= (TermMask & SHADING_TERM_HAIR_TRT) ? HAIR_COMPONENT_TRT : 0;

	FBxDFSample BxDFSample = (FBxDFSample)0;
	SampleHair(
		GBuffer.Roughness,
		GBuffer.BaseColor,
		GBuffer.Specular,
		Backlit,
		V,
		T,
		E.xy,
		E.zw,
		HairComponents,
		/* out */ BxDFSample.L,
		/* out */ BxDFSample.Weight);

	BxDFSample.PDF = 1.0; // TODO Guillaume: not used.
	BxDFSample.Term = TermMask; // TODO

	return BxDFSample;
}
#endif

// UE_DEPRECATED 5.7 - Deprecated by Substrate
FBxDFSample SampleBxDF(const uint TermMask, FGBufferData GBuffer, float3 V, float4 E)
{
	switch( GBuffer.ShadingModelID )
	{
		case SHADINGMODELID_DEFAULT_LIT:
		case SHADINGMODELID_SINGLELAYERWATER:
		case SHADINGMODELID_SUBSURFACE:
		case SHADINGMODELID_SUBSURFACE_PROFILE:
		case SHADINGMODELID_PREINTEGRATED_SKIN:
		case SHADINGMODELID_CLEAR_COAT:
		case SHADINGMODELID_TWOSIDED_FOLIAGE:
		case SHADINGMODELID_CLOTH:
		case SHADINGMODELID_EYE:
			return SampleDefaultLitBxDF(TermMask, GBuffer, V, E);
		case SHADINGMODELID_HAIR:
			return SampleHairBxDF(TermMask, GBuffer, V, E);
		default:
			return (FBxDFSample)0;
	}
}

bool SupportsSampleBxDF(uint ShadingModelID)
{
	switch (ShadingModelID)
	{
	case SHADINGMODELID_DEFAULT_LIT:
	case SHADINGMODELID_SINGLELAYERWATER:
	case SHADINGMODELID_SUBSURFACE:
	case SHADINGMODELID_SUBSURFACE_PROFILE:
	case SHADINGMODELID_PREINTEGRATED_SKIN:
	case SHADINGMODELID_CLEAR_COAT:
	case SHADINGMODELID_TWOSIDED_FOLIAGE:
	case SHADINGMODELID_CLOTH:
	case SHADINGMODELID_EYE:
	case SHADINGMODELID_HAIR:
		return true;
	default:
		return false;
	}
}
