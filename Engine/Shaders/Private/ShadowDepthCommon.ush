// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowDepthCommon.usf: Shared functionality for shadow depth rendering.
=============================================================================*/

#define INTERPOLATE_VF_ATTRIBUTES (!MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED || (REQUIRES_VF_ATTRIBUTES_FOR_CLIPPING && !POSITION_ONLY))
#define INTERPOLATE_POSITION (MATERIALBLENDING_MASKED)

/** Data passed from the vertex shader to the pixel shader for the shadow depth pass. */
struct FShadowDepthVSToPS
{
#if INTERPOLATE_VF_ATTRIBUTES
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
#endif

#if SUPPORT_SHADOW_DEPTH_SPOT
	float ShadowDepth : TEXCOORD6;
	float DepthBias : TEXCOORD8;
#endif

#if SUPPORT_SHADOW_DEPTH_POINT
	uint LayerIndex : SV_RenderTargetArrayIndex;
#endif

#if INTERPOLATE_POSITION
	// world space position needed in some cases, could be changed to be reconsructed from SvPosition (more tricky as this is not from camera perspective)
	float3 PixelPosition	: TEXCOORD7;
#endif

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	float3 PixelPositionExcludingWPO : TEXCOORD9;
#endif

#if SUPPORT_SHADOW_DEPTH_VSM
	nointerpolation uint PackedPageInfo : TEXCOORD8;
	float4 VirtualSmPageClip : SV_ClipDistance;
#endif
};
