// Copyright Epic Games, Inc. All Rights Reserved. 

/*=============================================================================
	ShadowDepthPixelShader.usf: Pixel shader for writing shadow depth.
=============================================================================*/

#if !MATERIAL_LWC_ENABLED
#define UE_DF_FORCE_FP32_OPS 1
#endif

// needs to before Common.usf
#define SHADOW_DEPTH_SHADER 1
#define USE_STENCIL_LOD_DITHER	0

#include "Nanite/NanitePackedNaniteView.ush"
#include "Common.ush"

// Reroute SceneTexturesStruct uniform buffer references to the shadow depth pass uniform buffer
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
#define PassStruct ShadowDepthPass
#define SceneTexturesStruct ShadowDepthPass.SceneTextures 
#else
#define PassStruct MobileShadowDepthPass
#define MobileSceneTextures MobileShadowDepthPass.SceneTextures 
#endif

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "ShadowDepthCommon.ush"

#if SUPPORT_SHADOW_DEPTH_VSM
#include "Nanite/NaniteDataDecode.ush"
#include "SceneData.ush"
#include "VirtualShadowMaps/VirtualShadowMapPageAccessCommon.ush"
#endif

void Main( 
	FShadowDepthVSToPS Inputs,
	in float4 SvPosition : SV_Position		// after all interpolators
#if SUPPORT_SHADOW_DEPTH_SPOT
	, out float OutDepth : SV_DEPTH
#endif
	)
{
	ResolvedView = ResolveView();

#if INTERPOLATE_VF_ATTRIBUTES
	
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Inputs.FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	#if INTERPOLATE_POSITION || (USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS)
	    {
			float4 ScreenPosition  = SvPositionToResolvedScreenPosition(SvPosition);  
			
			float3 TranslatedWorld = 
		#if INTERPOLATE_POSITION
			Inputs.PixelPosition.xyz;
		#else
			SvPositionToResolvedTranslatedWorld(SvPosition);
		#endif

			CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, 1, TranslatedWorld, 
		#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
				Inputs.PixelPositionExcludingWPO
		#else
				TranslatedWorld
		#endif			
			);
		}
	#else
		CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, 1);
	#endif

	// Evaluate the mask for masked materials
	GetMaterialClippingShadowDepth(MaterialParameters, PixelMaterialInputs);
#else
	ClipLODTransition(SvPosition.xy);
#endif

#if SUPPORT_SHADOW_DEPTH_SPOT
	const float InvMaxSubjectDepth = PassStruct.ShadowParams.w;
	Inputs.ShadowDepth = 1 - Inputs.ShadowDepth * InvMaxSubjectDepth;
	Inputs.ShadowDepth += Inputs.DepthBias;

	OutDepth = saturate(Inputs.ShadowDepth);
#endif

#if SUPPORT_SHADOW_DEPTH_VSM
	uint2 vAddress = (uint2)SvPosition.xy;
	float DeviceZ = SvPosition.z;

	FPageInfo PageInfo = UnpackPageInfo( Inputs.PackedPageInfo );
	FNaniteView NaniteView = UnpackNaniteView( PassStruct.PackedNaniteViews[ PageInfo.ViewId ] );

	// Note: not the usual UB
	FShadowPhysicalPage Page = ShadowDecodePageTable( PassStruct.VirtualSmPageTable[ CalcPageOffset( NaniteView.TargetLayerIndex, NaniteView.TargetMipLevel, vAddress >> VSM_LOG2_PAGE_SIZE ).GetResourceAddress() ] );

	if( Page.bThisLODValidForRendering)
	{
		uint2 pAddress = Page.PhysicalAddress * VSM_PAGE_SIZE + (vAddress & VSM_PAGE_SIZE_MASK);
		// If requested, render to the static page
		const int ArrayIndex = PageInfo.bStaticPage ? GetVirtualShadowMapStaticArrayIndex() : 0;
		InterlockedMax( PassStruct.OutDepthBufferArray[ uint3( pAddress, ArrayIndex ) ], asuint( DeviceZ ) );
	}
#endif
}
