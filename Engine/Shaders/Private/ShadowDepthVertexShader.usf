// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowDepthVertexShader.usf: Vertex & Geometry shader for writing shadow depth.
=============================================================================*/

// needs to before Common.usf
#define SHADOW_DEPTH_SHADER 1
#define USE_STENCIL_LOD_DITHER	0

// Need to get the definition of nanite packed view for generated unform buffer (which is pulled in by Common.ush)
#include "Nanite/NanitePackedNaniteView.ush"
#include "Common.ush"

// Reroute SceneTexturesStruct uniform buffer references to the shadow depth pass uniform buffer
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
#define PassStruct ShadowDepthPass
#define SceneTexturesStruct ShadowDepthPass.SceneTextures 
#else
#define PassStruct MobileShadowDepthPass
#define MobileSceneTextures MobileShadowDepthPass.SceneTextures 
#endif

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "ShadowDepthCommon.ush"

// 
#ifndef VF_USE_VSM_CONSTANT_BIAS
#define VF_USE_VSM_CONSTANT_BIAS								0
#endif

#if SUPPORT_SHADOW_DEPTH_VSM
#include "Nanite/NaniteDataDecode.ush"
#include "SceneData.ush"
#include "VirtualShadowMaps/VirtualShadowMapPageAccessCommon.ush"

/** 
 * Load physical page offset and perform the needed scale and bias to render the instance into the given physical page,
 * with the correct clipping set up.
 */
float4 ScaleBiasClipToPhysicalSmPage(FNaniteView NaniteView, float4 InPositionClip, inout float4 ClipPlanesInOut, FPageInfo PageInfo)
{
	float2 vUV = InPositionClip.xy * float2(0.5, -0.5) + 0.5 * InPositionClip.w;
	float2 vPixels = vUV * ( uint(VSM_VIRTUAL_MAX_RESOLUTION_XY) >> NaniteView.TargetMipLevel );

	float4 PositionClip = InPositionClip;
	{
		float2 Scale = exp2( -NaniteView.TargetMipLevel );
		PositionClip.xy *= Scale;
		PositionClip.xy += ( Scale * float2(1,-1) + float2(-1,1) ) * PositionClip.w;
	}

	{
		// Clip to uncached page rect
		uint4 PageRect = PassStruct.UncachedPageRectBounds[ NaniteView.TargetLayerIndex * VSM_MAX_MIP_LEVELS + NaniteView.TargetMipLevel ];

		float2 MinClip =  vPixels - ( PageRect.xy + 0 ) * VSM_PAGE_SIZE * PositionClip.w;
		float2 MaxClip = -vPixels + ( PageRect.zw + 1 ) * VSM_PAGE_SIZE * PositionClip.w;

		ClipPlanesInOut.xy = MinClip;
		ClipPlanesInOut.zw = MaxClip;
	}

	return PositionClip;
}

float4 TransformShadowVSM(inout float4 ClipPlanesInOut, FPageInfo PageInfo, float3 PointTranslatedWorld)
{
	// Load nanite view and perform transform to SM space.
	FNaniteView NaniteView = UnpackNaniteView(PassStruct.PackedNaniteViews[PageInfo.ViewId]);

	PointTranslatedWorld += DFFastSubtractDemote( NaniteView.PreViewTranslation, ResolvedView.PreViewTranslation );
	float4 PointClip = mul( float4( PointTranslatedWorld, 1 ), NaniteView.TranslatedWorldToClip );

	// Clamp the vertex to the near plane if it is in front of the near plane
	// This has problems if some vertices of a triangle get clamped and others do not, also causes artifacts with non-ortho projections
	if (PassStruct.bClampToNearPlane > 0 && PointClip.z > PointClip.w)
	{
		PointClip.z = 0.999999f;
		PointClip.w = 1.0f;
	}

	PointClip = ScaleBiasClipToPhysicalSmPage(NaniteView, PointClip, ClipPlanesInOut, PageInfo);

#if VF_USE_VSM_CONSTANT_BIAS
	{
		const float ConstantDepthBias = VertexFactoryGetNonNaniteVirtualShadowMapConstantDepthBias();
		PointClip.z = clamp(PointClip.z + ConstantDepthBias * NaniteView.ViewToClip[2][2], -PointClip.w, PointClip.w);
	}
#endif

	return PointClip;
}
#endif // SUPPORT_SHADOW_DEPTH_VSM

float ComputeDepthBiasDirectionalSpot(float4x4 WorldToShadowMatrix, float3 WorldVertexNormal)
{
	const float NoL = abs(dot(
		float3(WorldToShadowMatrix[0].z, WorldToShadowMatrix[1].z, WorldToShadowMatrix[2].z),
		WorldVertexNormal));

	const float MaxSlopeDepthBias = PassStruct.ShadowParams.z;
	const float Slope = clamp(abs(NoL) > 0 ? sqrt(saturate(1 - NoL*NoL)) / NoL : MaxSlopeDepthBias, 0, MaxSlopeDepthBias);
	
	const float SlopeDepthBias = PassStruct.ShadowParams.y;
	const float SlopeBias = SlopeDepthBias * Slope;

	const float ConstantDepthBias = PassStruct.ShadowParams.x;
	const float DepthBias = SlopeBias + ConstantDepthBias;

	return DepthBias;
}

// NOTE: Only for non-VSM directional lights
float4 TransformShadowDirectional(
	float4x4 WorldToClipMatrix, 
	float4x4 WorldToShadowMatrix, 
	float4 WorldPosition, 
	float3 WorldVertexNormal)
{
	float4 OutPosition = mul(WorldPosition, WorldToClipMatrix);

	// Clamp the vertex to the near plane if it is in front of the near plane
	// This has problems if some vertices of a triangle get clamped and others do not, also causes artifacts with non-ortho projections
	if (PassStruct.bClampToNearPlane > 0 && OutPosition.z > OutPosition.w)
	{
		OutPosition.z = 0.999999f;
		OutPosition.w = 1.0f;
	}

	const float DepthBias = ComputeDepthBiasDirectionalSpot(WorldToShadowMatrix, WorldVertexNormal);

	// Output linear, normalized depth
	const float InvMaxSubjectDepth = PassStruct.ShadowParams.w;
#if PLATFORM_NEEDS_PRECISE_SHADOW_DEPTH
	precise
#endif
	float AdjustedDepth = ( 1 - OutPosition.z * InvMaxSubjectDepth ) + DepthBias;
	OutPosition.z = AdjustedDepth;

	return OutPosition;
}

// Only for non-VSM spot lights
float4 TransformShadowSpot(
	float4x4 WorldToClipMatrix, 
	float4x4 WorldToShadowMatrix, 
	float4 WorldPosition, 
	float3 WorldVertexNormal,
	out float OutShadowDepth,
	out float OutDepthBias)
{
	float4 OutPosition = mul(WorldPosition, WorldToClipMatrix);
	OutShadowDepth = OutPosition.z;
	OutDepthBias = ComputeDepthBiasDirectionalSpot(WorldToShadowMatrix, WorldVertexNormal);
	return OutPosition;
}

// See EShadowDepthType
#define SHADOW_DEPTH_TYPE_VSM (1U)
#define SHADOW_DEPTH_TYPE_DIRECTIONAL (1U<<1)
#define SHADOW_DEPTH_TYPE_SPOT (1U<<2)
#define SHADOW_DEPTH_TYPE_ONEPASSPOINT (1U<<3)

uint bUseGpuSceneInstancing;
#if VERTEXSHADER && SUPPORT_SHADOW_DEPTH_POINT
uint LayerId;
#endif

void Main(
	FVertexFactoryInput Input,
	out FShadowDepthVSToPS OutParameters,
	out float4 OutPosition : SV_POSITION)
{
	ResolvedView = ResolveView();
	
	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPos = VertexFactoryGetWorldPosition(Input, VFIntermediates);
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	OutParameters.PixelPositionExcludingWPO = WorldPos.xyz;
#endif
	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPos.xyz, TangentToLocal);
	const float3 WorldNormal = VertexFactoryGetWorldNormal(Input, VFIntermediates);

	WorldPos.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	ApplyMaterialFirstPersonTransform(VertexParameters, WorldPos.xyz);

#if INTERPOLATE_VF_ATTRIBUTES
	// Masked materials need texture coords to clip
	OutParameters.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
#endif

#if INTERPOLATE_POSITION
	OutParameters.PixelPosition = WorldPos.xyz;
#endif

	OutPosition = float4(0, 0, 0, 0);

#if SUPPORT_SHADOW_DEPTH_VSM
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_VSM)
	{
		OutParameters.PackedPageInfo = 0;	
		OutParameters.VirtualSmPageClip = float4(1.0f, 1.0f, 1.0f, 1.0f);
		
		// Get the offset from which we loaded the instance ID
		uint InstanceIdIndex = VertexFactoryGetInstanceIdLoadIndex(VFIntermediates);
		OutParameters.PackedPageInfo = InstanceCulling.PageInfoBuffer[InstanceIdIndex];
		FPageInfo PageInfo = UnpackPageInfo(OutParameters.PackedPageInfo);
		OutPosition = TransformShadowVSM(OutParameters.VirtualSmPageClip, PageInfo, WorldPos.xyz);
	}
	else
	{
		// TODO: Cleaner way to initialize unused outputs...
		OutParameters.PackedPageInfo = 0;
		OutParameters.VirtualSmPageClip = float4(1, 1, 1, 1);
	}
#endif // SUPPORT_SHADOW_DEPTH_VSM

#if SUPPORT_SHADOW_DEPTH_DIRECTIONAL
	// Non-VSM directional lights
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_DIRECTIONAL)
	{
		OutPosition = TransformShadowDirectional(
			PassStruct.ProjectionMatrix,
			PassStruct.ViewMatrix,
			WorldPos, 
			WorldNormal);
	}
#endif // SUPPORT_SHADOW_DEPTH_DIRECTIONAL

#if SUPPORT_SHADOW_DEPTH_SPOT
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_SPOT)
	{
		OutPosition = TransformShadowSpot(
			PassStruct.ProjectionMatrix,
			PassStruct.ViewMatrix,
			WorldPos,
			WorldNormal,
			OutParameters.ShadowDepth,
			OutParameters.DepthBias);
	}
#endif // SUPPORT_SHADOW_DEPTH_SPOT

#if SUPPORT_SHADOW_DEPTH_POINT
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_ONEPASSPOINT)
	{
		OutParameters.LayerIndex = bUseGpuSceneInstancing ? VertexFactoryGetViewIndex(VFIntermediates) : LayerId;
		OutPosition = mul(WorldPos, PassStruct.ShadowViewProjectionMatrices[OutParameters.LayerIndex]);
	}
	else
	{
		OutParameters.LayerIndex = 0;
	}
#endif // SUPPORT_SHADOW_DEPTH_POINT
}

#if POSITION_ONLY
void PositionOnlyMain(
	in FPositionAndNormalOnlyVertexFactoryInput Input,
	out FShadowDepthVSToPS OutParameters,
	out float4 OutPosition : SV_POSITION)
{
	ResolvedView = ResolveView();

	float4 WorldPos = VertexFactoryGetWorldPosition(Input);
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	OutParameters.PixelPositionExcludingWPO = WorldPos.xyz;
#endif

	float3 WorldNormal = VertexFactoryGetWorldNormal(Input);

#if INTERPOLATE_VF_ATTRIBUTES
	OutParameters.FactoryInterpolants = (FVertexFactoryInterpolantsVSToPS)0;
#endif

#if INTERPOLATE_POSITION
	OutParameters.PixelPosition = WorldPos.xyz;
#endif

	OutPosition = float4(0, 0, 0, 0);

#if SUPPORT_SHADOW_DEPTH_VSM
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_VSM)
	{
		OutParameters.PackedPageInfo = 0;
		OutParameters.VirtualSmPageClip = float4(1.0f, 1.0f, 1.0f, 1.0f);
		
		// Get the offset from which we loaded the instance ID
		uint InstanceIdIndex = VertexFactoryGetInstanceIdLoadIndex(Input);
		OutParameters.PackedPageInfo = InstanceCulling.PageInfoBuffer[InstanceIdIndex];
		FPageInfo PageInfo = UnpackPageInfo(OutParameters.PackedPageInfo);
		OutPosition = TransformShadowVSM(OutParameters.VirtualSmPageClip, PageInfo, WorldPos.xyz);
	}
	else
	{
		// TODO: Cleaner way to initialize unused outputs...
		OutParameters.PackedPageInfo = 0;
		OutParameters.VirtualSmPageClip = float4(1, 1, 1, 1);
	}	
#endif // SUPPORT_SHADOW_DEPTH_VSM

#if	SUPPORT_SHADOW_DEPTH_DIRECTIONAL
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_DIRECTIONAL)
	{
		OutPosition = TransformShadowDirectional(
			PassStruct.ProjectionMatrix,
			PassStruct.ViewMatrix,
			WorldPos, 
			WorldNormal);
	}
#endif // SUPPORT_SHADOW_DEPTH_DIRECTIONAL

#if SUPPORT_SHADOW_DEPTH_POINT
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_ONEPASSPOINT)
	{
		OutParameters.LayerIndex = bUseGpuSceneInstancing ? VertexFactoryGetViewIndex(Input) : LayerId;
		OutPosition = mul(WorldPos, PassStruct.ShadowViewProjectionMatrices[OutParameters.LayerIndex]);
	}
	else
	{
		OutParameters.LayerIndex = 0;
	}	
#endif // SUPPORT_SHADOW_DEPTH_POINT

#if SUPPORT_SHADOW_DEPTH_SPOT
	if (PassStruct.ShadowDepthType == SHADOW_DEPTH_TYPE_SPOT)
	{
		OutPosition = TransformShadowSpot(
			PassStruct.ProjectionMatrix,
			PassStruct.ViewMatrix,
			WorldPos,
			WorldNormal,
			OutParameters.ShadowDepth,
			OutParameters.DepthBias);
	}
#endif // SUPPORT_SHADOW_DEPTH_SPOT
}


#endif // POSITION_ONLY
