// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowFactorsUpsampleCommon.ush
=============================================================================*/

#ifndef SHADOW_FACTORS_UPSAMPLE_REQUIRED
#error "Must define SHADOW_FACTORS_UPSAMPLE_REQUIRED"
#endif

#include "Common.ush"
#include "ComputeShaderUtils.ush"

void UpsampleShadowFactors(
	float4 SVPos, 
	int4 ScissorRectMinAndSize,
	float OneOverDownsampleFactor, 
	float ShadowFactorsMinDepth, 
	float ShadowFactorsMaxDepth, 
	Texture2D ShadowFactorsTexture, 
	SamplerState ShadowFactorsSampler,
	float2 ShadowFactorsUVBilinearMax,
	out float ShadowFactor, 
	out float SceneDepth,
	uint EyeIndex)
{
	float2 BufferUV = SvPositionToBufferUV(SVPos);

	// ShadowFactors is computed at 0,0 regardless of viewrect min
	float2 ShadowFactorsUV = BufferUV - ScissorRectMinAndSize.xy * View.BufferSizeAndInvSize.zw;
#if MOBILE_MULTI_VIEW
	SceneDepth = CalcSceneDepth(BufferUV, EyeIndex);
#else
	SceneDepth = CalcSceneDepth(BufferUV);
#endif

#define BILATERAL_UPSAMPLE 1
#if BILATERAL_UPSAMPLE && SHADOW_FACTORS_UPSAMPLE_REQUIRED
	float2 LowResBufferSize = floor(View.BufferSizeAndInvSize.xy * OneOverDownsampleFactor);
	float2 LowResTexelSize = 1.0f / LowResBufferSize;

	// since DF shadow rays are traced from the position of the full-res texel at the top left of low-res tile
	// we need to use different UVs for calculating bilinear weights and sampling the 4 texels

	float2 Corner00UVBase = floor(ShadowFactorsUV * LowResBufferSize) / LowResBufferSize;

	// Calculate bilinear weights using the same UVs that rays were traced from
	float2 Corner00UV = Corner00UVBase + 0.5f * View.BufferSizeAndInvSize.zw;
	float2 BilinearWeights = saturate((ShadowFactorsUV - Corner00UV) * LowResBufferSize);

	// Sample low res texture using UVs at center of low-res texels since we are interpolating manually
	float2 Corner00SampleUV = Corner00UVBase + 0.5f * LowResTexelSize;

	float2 TextureValues00 = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, min(Corner00SampleUV, ShadowFactorsUVBilinearMax), 0).xy;
	float2 TextureValues10 = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, min(Corner00SampleUV + float2(LowResTexelSize.x, 0), ShadowFactorsUVBilinearMax), 0).xy;
	float2 TextureValues01 = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, min(Corner00SampleUV + float2(0, LowResTexelSize.y), ShadowFactorsUVBilinearMax), 0).xy;
	float2 TextureValues11 = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, min(Corner00SampleUV + LowResTexelSize, ShadowFactorsUVBilinearMax), 0).xy;

	float4 CornerWeights = float4(
		(1 - BilinearWeights.y) * (1 - BilinearWeights.x), 
		(1 - BilinearWeights.y) * BilinearWeights.x,
		BilinearWeights.y * (1 - BilinearWeights.x),
		BilinearWeights.y * BilinearWeights.x);

	float Epsilon = .0001f;

	float4 ShadowFactors = float4(TextureValues00.x, TextureValues10.x, TextureValues01.x, TextureValues11.x);
	float4 CornerDepths = abs(float4(TextureValues00.y, TextureValues10.y, TextureValues01.y, TextureValues11.y));

	// Ignore pixels marked as non-shadowed if they lie outside the DF shadows range.
	// Shadowed pixels outside the range are marked as valid, these could be from heightfield traces.
	float4 CornerDepthsValid = 
		or(
			and(ShadowFactorsMinDepth.xxxx < CornerDepths, CornerDepths < ShadowFactorsMaxDepth.xxxx),
			ShadowFactors < 1.0
		);
	float4 DepthWeights = 1.0f / (abs(CornerDepths - SceneDepth.xxxx) + Epsilon);
	float4 FinalWeights = CornerWeights * CornerDepthsValid * DepthWeights;
	float TotalWeight = dot(FinalWeights, 1);

	float InterpolatedResult = 1;
	if (TotalWeight != 0)
	{
		InterpolatedResult = dot(ShadowFactors, FinalWeights) / TotalWeight;
	}

	ShadowFactor = InterpolatedResult;
#else
	ShadowFactor = Texture2DSampleLevel(ShadowFactorsTexture, ShadowFactorsSampler, ShadowFactorsUV, 0).x;
#endif
}

void UpsampleShadowFactors(
	float4 SVPos,
	int4 ScissorRectMinAndSize,
	float OneOverDownsampleFactor,
	float ShadowFactorsMinDepth,
	float ShadowFactorsMaxDepth,
	Texture2D ShadowFactorsTexture,
	SamplerState ShadowFactorsSampler,
	float2 ShadowFactorsUVBilinearMax,
	out float ShadowFactor,
	out float SceneDepth)
{
	UpsampleShadowFactors(SVPos, ScissorRectMinAndSize, OneOverDownsampleFactor, ShadowFactorsMinDepth, ShadowFactorsMaxDepth, ShadowFactorsTexture, ShadowFactorsSampler, ShadowFactorsUVBilinearMax, ShadowFactor, SceneDepth, 0);
}
