// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowFilteringCommon.usf: Contains functions to filter a shadowmap, shared between forward/deferred shading.
=========================================================================*/

#pragma once

#include "PixelQuadMessagePassing.ush"
#include "FastMath.ush"
#include "TransmissionCommon.ush"

struct FPCFSamplerSettings
{
	Texture2D		ShadowDepthTexture;
	SamplerState	ShadowDepthTextureSampler;

	//XY - Pixel size of shadowmap
	//ZW - Inverse pixel size of shadowmap
	float4			ShadowBufferSize;

	// SceneDepth in lightspace.
	float			SceneDepth;

	float			TransitionScale;

	// set by the caller, constant for the code so only one code branch should be compiled
	bool			bSubsurface;

	// Whether to treat shadow depths near 1 as unshadowed.  This is useful when the shadowmap does not contain all the points being shadowed.
	bool			bTreatMaxDepthUnshadowed;

	// only used if bSubsurface is true
	float			DensityMulConstant;

	// only used if bSubsurface is true
	float2			ProjectionDepthBiasParameters;
};

// linear PCF, input 3x3
// @param Values0 in row 0 from left to right: x,y,z,w
// @param Values1 in row 1 from left to right: x,y,z,w
// @param Values2 in row 2 from left to right: x,y,z,w
float PCF2x2(float2 Fraction, float3 Values0, float3 Values1, float3 Values2)
{
	float3 Results;

	Results.x = Values0.x * (1.0f - Fraction.x);
	Results.y = Values1.x * (1.0f - Fraction.x);
	Results.z = Values2.x * (1.0f - Fraction.x);
	Results.x += Values0.y;
	Results.y += Values1.y;
	Results.z += Values2.y;
	Results.x += Values0.z * Fraction.x;
	Results.y += Values1.z * Fraction.x;
	Results.z += Values2.z * Fraction.x;

	return saturate(0.25f * dot(Results, half3(1.0f - Fraction.y, 1.0f, Fraction.y)));
}

// linear PCF, input 4x4
// @param Values0 in row 0 from left to right: x,y,z,w
// @param Values1 in row 1 from left to right: x,y,z,w
// @param Values2 in row 2 from left to right: x,y,z,w
// @param Values3 in row 3 from left to right: x,y,z,w
float PCF3x3(float2 Fraction, float4 Values0, float4 Values1, float4 Values2, float4 Values3)
{
	float4 Results;

	Results.x = Values0.x * (1.0f - Fraction.x);
	Results.y = Values1.x * (1.0f - Fraction.x);
	Results.z = Values2.x * (1.0f - Fraction.x);
	Results.w = Values3.x * (1.0f - Fraction.x);
	Results.x += Values0.y;
	Results.y += Values1.y;
	Results.z += Values2.y;
	Results.w += Values3.y;
	Results.x += Values0.z;
	Results.y += Values1.z;
	Results.z += Values2.z;
	Results.w += Values3.z;
	Results.x += Values0.w * Fraction.x;
	Results.y += Values1.w * Fraction.x;
	Results.z += Values2.w * Fraction.x;
	Results.w += Values3.w * Fraction.x;

	return saturate(dot(Results, float4(1.0f - Fraction.y, 1.0f, 1.0f, Fraction.y)) * (1.0f / 9.0f));
}


// linear PCF, input 4x4
// using Gather: xyzw in counter clockwise order starting with the sample to the lower left of the queried location
// @param Values0 left top
// @param Values1 right top
// @param Values2 left bottom
// @param Values3 right bottom
float PCF3x3gather(float2 Fraction, float4 Values0, float4 Values1, float4 Values2, float4 Values3)
{
	float4 Results;

	Results.x = Values0.w * (1.0 - Fraction.x);
	Results.y = Values0.x * (1.0 - Fraction.x);
	Results.z = Values2.w * (1.0 - Fraction.x);
	Results.w = Values2.x * (1.0 - Fraction.x);
	Results.x += Values0.z;
	Results.y += Values0.y;
	Results.z += Values2.z;
	Results.w += Values2.y;
	Results.x += Values1.w;
	Results.y += Values1.x;
	Results.z += Values3.w;
	Results.w += Values3.x;
	Results.x += Values1.z * Fraction.x;
	Results.y += Values1.y * Fraction.x;
	Results.z += Values3.z * Fraction.x;
	Results.w += Values3.y * Fraction.x;

	return dot( Results, float4( 1.0 - Fraction.y, 1.0, 1.0, Fraction.y) * ( 1.0 / 9.0) );
}

// horizontal PCF, input 6x2
float2 HorizontalPCF5x2(float2 Fraction, float4 Values00, float4 Values20, float4 Values40)
{
	float Results0;
	float Results1;

	Results0 = Values00.w * (1.0 - Fraction.x);
	Results1 = Values00.x * (1.0 - Fraction.x);
	Results0 += Values00.z;
	Results1 += Values00.y;
	Results0 += Values20.w;
	Results1 += Values20.x;
	Results0 += Values20.z;
	Results1 += Values20.y;
	Results0 += Values40.w;
	Results1 += Values40.x;
	Results0 += Values40.z * Fraction.x;
	Results1 += Values40.y * Fraction.x;

	return float2(Results0, Results1);
}

// lowest quality ith PCF
float PCF1x1(float2 Fraction, float4 Values00)
{
	float2 HorizontalLerp00 = lerp(Values00.wx, Values00.zy, Fraction.xx);

	return lerp(HorizontalLerp00.x, HorizontalLerp00.y, Fraction.y);
}

float4 CalculateShadowVisibilityTransmittanceFactor(float4 ShadowmapDepth, FPCFSamplerSettings Settings)
{
	if (Settings.bSubsurface)
	{
		float4 OccluderDistance = max(Settings.SceneDepth - ShadowmapDepth, 0);
		float4 Transmittance = GetSubSurfaceTransmission(OccluderDistance, Settings.DensityMulConstant);
		// Never shadow from depths that were never written to (max depth value)
		return select(ShadowmapDepth > .99f, 1.0f, Transmittance);
	}
	else
	{
		// The standard comparison is SceneDepth < ShadowmapDepth
		// Using a soft transition based on depth difference
		// Offsets shadows a bit but reduces self shadowing artifacts considerably
		float TransitionScale = Settings.TransitionScale;

		// Unoptimized Math: saturate((ShadowmapDepth - Settings.SceneDepth) * TransitionScale + 1);
		// Rearranged the math so that per pixel constants can be optimized from per sample constants.
		float ConstantFactor = (Settings.SceneDepth * TransitionScale - 1);
		float4 ShadowFactor = saturate(ShadowmapDepth * TransitionScale - ConstantFactor);

		FLATTEN
		if (Settings.bTreatMaxDepthUnshadowed)
		{
			ShadowFactor = saturate(ShadowFactor + (ShadowmapDepth > .99f));
		}

		return ShadowFactor;
	}
}

float3 CalculateShadowVisibilityTransmittanceFactor(float3 ShadowmapDepth, FPCFSamplerSettings Settings)
{
	if (Settings.bSubsurface)
	{
		// Determine the distance that the light traveled through the subsurface object
		// This assumes that anything between this subsurface pixel and the light was also a subsurface material,
		// As a result, subsurface materials receive leaked light based on how transparent they are
		float3 Thickness = max(Settings.SceneDepth - ShadowmapDepth, 0);
		float3 OpticalDepth = Thickness * Settings.DensityMulConstant;
		float3 Transmittance = saturate(FastExp(-OpticalDepth));
		// Never shadow from depths that were never written to (max depth value)
		return select(ShadowmapDepth > .99f, 1.0, Transmittance);
	}
	else
	{
		// The standard comparison is Settings.SceneDepth < ShadowmapDepth
		// Using a soft transition based on depth difference
		// Offsets shadows a bit but reduces self shadowing artifacts considerably
		float TransitionScale = Settings.TransitionScale;

		// Unoptimized Math: saturate((ShadowmapDepth - Settings.SceneDepth) * TransitionScale + 1);
		// Rearranged the math so that per pixel constants can be optimized from per sample constants.
		float ConstantFactor = (Settings.SceneDepth * TransitionScale - 1);
		float3 ShadowFactor = saturate(ShadowmapDepth * TransitionScale - ConstantFactor);

		FLATTEN
		if (Settings.bTreatMaxDepthUnshadowed)
		{
			ShadowFactor = saturate(ShadowFactor + (ShadowmapDepth > .99f));
		}

		return ShadowFactor;
	}
}

float3 FetchRowOfThree(float2 Sample00TexelCenter, float VerticalOffset, FPCFSamplerSettings Settings)
{
	float3 Values;
	Values.x = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(0, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.y = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(1, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.z = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(2, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;	
	return CalculateShadowVisibilityTransmittanceFactor(Values, Settings);
}

float4 FetchRowOfFour(float2 Sample00TexelCenter, float VerticalOffset, FPCFSamplerSettings Settings)
{
	float4 Values;
	Values.x = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(0, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.y = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(1, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.z = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(2, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.w = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(3, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	return CalculateShadowVisibilityTransmittanceFactor(Values, Settings);
}

float3 FetchRowOfThreeAfterFour(float2 Sample00TexelCenter, float VerticalOffset, FPCFSamplerSettings Settings)
{
	float3 Values;
	Values.x = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(4, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.y = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(5, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	Values.z = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, (Sample00TexelCenter + float2(6, VerticalOffset)) * Settings.ShadowBufferSize.zw, 0).r;
	return CalculateShadowVisibilityTransmittanceFactor(Values, Settings);
}

float Manual3x3PCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	// medium quality, 4x4 samples, using and not using gather4
	{
		float2 TexelPos = ShadowPosition * Settings.ShadowBufferSize.xy - 0.5f;	// bias to be consistent with texture filtering hardware
		float2 Fraction = frac(TexelPos);
		float2 TexelCorner = floor(TexelPos);	// bias to get reliable texel center content

		// Gather4 samples "at the following locations: (-,+),(+,+),(+,-),(-,-), where the magnitude of the deltas are always half a texel" - DX11 Func. Spec.
		// So we need to offset to the centre of the 2x2 grid we want to sample.
		float2 SamplePos = (TexelCorner + 1.0f) * Settings.ShadowBufferSize.zw;	// bias to get reliable texel center content
		float4 SampleValues0 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(-1, -1)), Settings);
		float4 SampleValues1 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(1, -1)), Settings);
		float4 SampleValues2 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(-1, 1)), Settings);
		float4 SampleValues3 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(1, 1)), Settings);
		return PCF3x3gather(Fraction, SampleValues0, SampleValues1, SampleValues2, SampleValues3);
	}
}

float Manual5x5PCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	// high quality, 6x6 samples, using gather4
	float2 TexelPos = ShadowPosition * Settings.ShadowBufferSize.xy - 0.5f;	// bias to be consistent with texture filtering hardware
	float2 Fraction = frac(TexelPos);
	// Gather4 samples "at the following locations: (-,+),(+,+),(+,-),(-,-), where the magnitude of the deltas are always half a texel" - DX11 Func. Spec.
	// So we need to offset to the centre of the 2x2 grid we want to sample.
	float2 SamplePos = (floor(TexelPos) + 1.0f) * Settings.ShadowBufferSize.zw;	// bias to get reliable texel center content

	float4 Values00 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(-2, -2)), Settings);
	float4 Values20 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(0, -2)), Settings);
	float4 Values40 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(2, -2)), Settings);

	float2 Row0 = HorizontalPCF5x2(Fraction, Values00, Values20, Values40);
	float Results = Row0.x * (1.0f - Fraction.y) + Row0.y;

	float4 Values02 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(-2, 0)), Settings);
	float4 Values22 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(0, 0)), Settings);
	float4 Values42 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(2, 0)), Settings);

	float2 Row1 = HorizontalPCF5x2(Fraction, Values02, Values22, Values42);
	Results += Row1.x + Row1.y;

	float4 Values04 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(-2, 2)), Settings);
	float4 Values24 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(0, 2)), Settings);
	float4 Values44 = CalculateShadowVisibilityTransmittanceFactor(Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, SamplePos, int2(2, 2)), Settings);

	float2 Row2 = HorizontalPCF5x2(Fraction, Values04, Values24, Values44);
	Results += Row2.x + Row2.y * Fraction.y;

	return 0.04f * Results;
}

// break this out for forward rendering as it's not part of ManualPCF's set of shadowquality settings.
float Manual2x2PCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	float2 TexelPos = ShadowPosition * Settings.ShadowBufferSize.xy;
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos) + 0.5f;	// bias to get reliable texel center content
	
	float2 Sample00TexelCenter = TexelCenter - float2(1, 1);

	float3 SamplesValues0 = FetchRowOfThree(Sample00TexelCenter, 0, Settings);
	float3 SamplesValues1 = FetchRowOfThree(Sample00TexelCenter, 1, Settings);
	float3 SamplesValues2 = FetchRowOfThree(Sample00TexelCenter, 2, Settings);
	
	return PCF2x2(Fraction, SamplesValues0, SamplesValues1, SamplesValues2);
}

// break this out for forward rendering as it's not part of ManualPCF's set of shadowquality settings.
float Manuax2PCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	float2 TexelPos = ShadowPosition * Settings.ShadowBufferSize.xy;
	float2 Fraction = frac(TexelPos);
	float2 TexelCenter = floor(TexelPos) + 0.5f;	// bias to get reliable texel center content
	
	float2 Sample00TexelCenter = TexelCenter - float2(1, 1);

	float3 SamplesValues0 = FetchRowOfThree(Sample00TexelCenter, 0, Settings);
	float3 SamplesValues1 = FetchRowOfThree(Sample00TexelCenter, 1, Settings);
	float3 SamplesValues2 = FetchRowOfThree(Sample00TexelCenter, 2, Settings);
	
	return PCF2x2(Fraction, SamplesValues0, SamplesValues1, SamplesValues2);
}

float ManualNoFiltering(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	// very low quality but very good performance, useful to profile, 1 sample, not using gather4
	return CalculateShadowVisibilityTransmittanceFactor(Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, ShadowPosition, 0).rrr, Settings).r;
}

float Manual1x1PCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
	float2 TexelPos = ShadowPosition * Settings.ShadowBufferSize.xy - 0.5f;	// bias to be consistent with texture filtering hardware
	float2 Fraction = frac(TexelPos);

	// using Gather: xyzw in counter clockwise order starting with the sample to the lower left of the queried location
	float4 Samples;

	// Gather4 samples "at the following locations: (-,+),(+,+),(+,-),(-,-), where the magnitude of the deltas are always half a texel" - DX11 Func. Spec.
	// So we need to offset to the centre of the 2x2 grid we want to sample.
	float2 QuadCenter = floor(TexelPos) + 1.0f;	
	Samples = Settings.ShadowDepthTexture.Gather(Settings.ShadowDepthTextureSampler, QuadCenter * Settings.ShadowBufferSize.zw);

	float4 Values00 = CalculateShadowVisibilityTransmittanceFactor(Samples, Settings);
	return PCF1x1(Fraction, Values00);
}


float ManualPCF(float2 ShadowPosition, FPCFSamplerSettings Settings)
{
#if SHADOW_QUALITY == 1
	return ManualNoFiltering(ShadowPosition, Settings);
#elif SHADOW_QUALITY == 2
	// low quality, 2x2 samples, using and not using gather4
	return Manual1x1PCF(ShadowPosition, Settings);
#elif SHADOW_QUALITY == 3
	return Manual3x3PCF(ShadowPosition, Settings);
#else
	return Manual5x5PCF(ShadowPosition, Settings);
#endif
}
