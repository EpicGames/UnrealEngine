// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowPercentageCloserFiltering.usf: Contains functions to do percentage closer soft shadow sampling.
=========================================================================*/

#include "ShadowFilteringCommon.ush"
#include "MonteCarlo.ush"
#include "SobolRandom.ush"


// CONFIGURATION OF PCSS
// -----------------------------------------------------------------------

// Whether to compile some debugging utilitaries.
#define PCSS_DEBUG_UTILITITARIES 0

// Whether to debug pixel where they early return.
#define PCSS_DEBUG_EARLY_RETURN 0

// Wheather some computation should be shared across 2x2 pixel quad.
//  0: disabled.
//  1: share occluder search result.
//  2: share occluder search and PCF results.
#define PCSS_SHARE_PER_PIXEL_QUAD 2

// Wheather to set a maximum depth bias.
#define PCSS_MAX_DEPTH_BIAS 1

// Idea of the experiment to turn on.
#define PCSS_ANTI_ALIASING_METHOD 2

// PCF experiment to solve translucent shadow artifacts.
//  0: Dummy PCF.
//  1: Cone traced occluder sums with occluder distance based weighting.
#define PCSS_PCF_EXPERIMENT 0

// Wheather to enable the sharpening filter after PCF for sharper edges than the shadow map resolution.
#define PCSS_ENABLE_POST_PCF_SHARPENING 1

// Blocker search samples
#define PCSS_SEARCH_BITS 4
#define PCSS_SEARCH_SAMPLES (1 << PCSS_SEARCH_BITS)

// Shadow filtering samples
#define PCSS_SAMPLE_BITS 5
#define PCSS_SAMPLES (1 << PCSS_SAMPLE_BITS)


// POISSON KERNEL
// -----------------------------------------------------------------------

struct FPCSSSamplerSettings
{
	Texture2D ShadowDepthTexture;
	SamplerState ShadowDepthTextureSampler;

	//XY - Pixel size of shadowmap
	//ZW - Inverse pixel size of shadowmap
	float4 ShadowBufferSize;

	// Offset and size of the shadow tile within the shadow depth texture.
	float4 ShadowTileOffsetAndSize;

	// SceneDepth in shadow view space.
	float SceneDepth;

	// Transition scale (TODO: Could probably remove this).
	float TransitionScale;

	// Size of a SourceRadius-sized sphere in shadow space
	float ProjectedSourceRadius;
	
	// Tan(0.5 * Light Source Angle) in the shadow view space.
	float TanLightSourceAngle;
	
	// Maximum kernel size in the tile's UV space.
	float MaxKernelSize;
	
	// Pixel's postion for random number generation.
	float2 SvPosition;

	// Thread's pixel quad message passing context.
	FPQMPContext PQMPContext;

	// Viewport UV for debuging purposes.
	float2 DebugViewportUV;
};


// PCSS SPECIFIC UTILITARY FUNCTIONS
// -----------------------------------------------------------------------

float2x2 GenerateScale2x2Matrix(float2 MajorAxis, float MajorMinusMinorScale, float MinorScale)
{
	return float2x2(
		MinorScale + MajorMinusMinorScale * MajorAxis.x * MajorAxis.x, MajorMinusMinorScale * MajorAxis.y * MajorAxis.x,
		MajorMinusMinorScale * MajorAxis.x * MajorAxis.y, MinorScale + MajorMinusMinorScale * MajorAxis.y * MajorAxis.y);
}

float2x2 GenerateDirectionalScale2x2Matrix(float2 Direction, float ScaleMinusOne)
{
	return float2x2(
		1 + ScaleMinusOne * Direction.x * Direction.x, ScaleMinusOne * Direction.y * Direction.x,
		ScaleMinusOne * Direction.x * Direction.y, 1 + ScaleMinusOne * Direction.y * Direction.y);
}

#if PCSS_DEBUG_UTILITITARIES

// Function to debug the direction of ellipse major and minor axes in the shadow map UV space.
float PCSSDebugUVDir(float2 v)
{
	return frac(atan2(v.x, v.y) / (PI)+0.25);
}

#endif //PCSS_DEBUG_UTILITITARIES


// PCSS FILTER
// -----------------------------------------------------------------------

float DirectionalPCSS(FPCSSSamplerSettings Settings, float2 ShadowPosition, float3 ShadowPositionDDX, float3 ShadowPositionDDY)
{
	float3 DepthBiasPlaneNormal = cross(ShadowPositionDDX, ShadowPositionDDY);
#if PCSS_MAX_DEPTH_BIAS
	float DepthBiasFactor = 1 / max(abs(DepthBiasPlaneNormal.z), length(DepthBiasPlaneNormal) * 0.0872665);
#else
	float DepthBiasFactor = 1 / abs(DepthBiasPlaneNormal.z);
#endif
	float2 DepthBiasDotFactors = DepthBiasPlaneNormal.xy * DepthBiasFactor;

	float2 ShadowTexelSizeX = Settings.ShadowBufferSize.zw / Settings.ShadowTileOffsetAndSize.zw; //TODO: remove truncation.
	const float MinFilterSize = max(ShadowTexelSizeX.x, ShadowTexelSizeX.y);
	float PCSSMinFilterSize = MinFilterSize;
	float PCFMinFilterSize = MinFilterSize;

	// dot(surface's normal, light direction).
	float3 NormalizedDepthBiasPlaneNormal = normalize(DepthBiasPlaneNormal);
	float DotNormal = NormalizedDepthBiasPlaneNormal.z;

	uint2 SobolRandom = SobolIndex(SobolPixel(Settings.SvPosition.xy), View.StateFrameIndexMod8);
	float RandomFilterScale = 0.75;
	float2x2 PerPixelRotationMatrix = float2x2(1, 0, 0, 1);

#if PCSS_ANTI_ALIASING_METHOD == 1 || PCSS_ANTI_ALIASING_METHOD == 2 // Dilate occluder search.
	PCSSMinFilterSize *= 3;
#endif

#if SPOT_LIGHT_PCSS
	float SearchRadius = Settings.ProjectedSourceRadius;
#else
	float SearchRadius = Settings.SceneDepth * Settings.TanLightSourceAngle;
#endif
	SearchRadius = clamp(PCSSMinFilterSize, Settings.MaxKernelSize, SearchRadius);

#if PCSS_ANTI_ALIASING_METHOD == 6 // Experience 6: anysotropy.
	float2 ShadowPositionDx = ddx(ShadowPosition);
	float2 ShadowPositionDy = ddy(ShadowPosition);
	float2 ShadowAnisotropicVector = ShadowPositionDx + ShadowPositionDy;
	float2 ShadowPositionDelta = sqrt(float2(
		ShadowPositionDx.x * ShadowPositionDx.x + ShadowPositionDy.x * ShadowPositionDy.x,
		ShadowPositionDx.y * ShadowPositionDx.y + ShadowPositionDy.y * ShadowPositionDy.y));

	float MajorAnisotropy = max(ShadowPositionDelta.x, ShadowPositionDelta.y);
	float MinorAnisotropy = min(ShadowPositionDelta.x, ShadowPositionDelta.y);

	float MinorAnisotropyFactor = clamp(MinorAnisotropy / SearchRadius, 1, 7);
	float MajorAnisotropyFactor = clamp(MajorAnisotropy / SearchRadius, 1, 7);
	//if (View.GeneralPurposeTweak < 1.5) {
	//	MajorAnisotropyFactor = 1;
	//	MinorAnisotropyFactor = 1;
	//}
	//if (View.GeneralPurposeTweak < 2.5) {
	//	MajorAnisotropyFactor = clamp(length(ShadowAnisotropicVector) / SearchRadius, 1, 7);
	//	MinorAnisotropyFactor = 1;
	//}
	//return MajorAnisotropyFactor - MinorAnisotropyFactor;
	//return (MinorAnisotropyFactor - 1);

	float2 ShadowAnisotropicDir = normalize(ShadowAnisotropicVector);
	float2x2 AnysotropticProjectionMatrix = GenerateScale2x2Matrix(
		ShadowAnisotropicDir, MajorAnisotropyFactor - MinorAnisotropyFactor, MinorAnisotropyFactor);

	PerPixelRotationMatrix = AnysotropticProjectionMatrix;
#endif

	// Search for an average occluder depth.
	float DepthSum = 0;
	float DepthSquaredSum = 0;
	float SumWeight = 0;
	float DiscardedSampleCount = 0;
	float2 UVOffsetSum = float2(0, 0);
	float2 UVOffsetSquareSum = float2(0, 0);
	float UVCrossSum = 0;
	for (int i = 0; i < PCSS_SEARCH_SAMPLES; i++)
	{
		float2 E = SobolIndex(SobolRandom ^ uint2(0x4B05, 0xB0CD), i << 3, PCSS_SEARCH_BITS + 3) / float(0x10000);
		float2 PCSSSample = UniformSampleDiskConcentricApprox(E);
		float2 SampleUVOffset = PCSSSample * SearchRadius;
		float2 SampleUV = ShadowPosition + SampleUVOffset * Settings.ShadowTileOffsetAndSize.zw;

		float ShadowDepth = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, SampleUV, 0).r;
		float ShadowDepthCompare = Settings.SceneDepth - ShadowDepth;

		float SampleDepthBias = max(dot(DepthBiasDotFactors, SampleUVOffset), 0);

		// TODO: Cone trace to early return more often on SumWeight == 0?
		if (ShadowDepthCompare > SampleDepthBias)
		{
			DepthSum += ShadowDepth;
			DepthSquaredSum += ShadowDepth * ShadowDepth;
			SumWeight += 1;
			float2 UV = PCSSSample;
			UVOffsetSum += UV;
			UVOffsetSquareSum += UV * UV;
			UVCrossSum += UV.x * UV.y;
		}
		else if (ShadowDepthCompare > 0)
		{
			DiscardedSampleCount += 1;
		}
	}

#if PCSS_SHARE_PER_PIXEL_QUAD // Pixel quad share setup.
	const float MaxTexelShare = 1;
	float2 MaxDerivative = max(abs(ddx(ShadowPosition)), abs(ddy(ShadowPosition)));
	float DoPerQuad = 0.8 * max(1 - max(MaxDerivative.x, MaxDerivative.y) / (MinFilterSize * MaxTexelShare), 0);
#else
	float DoPerQuad = 0;
#endif 

#if PCSS_SHARE_PER_PIXEL_QUAD >= 1 // Share occluder search computation per quad.
	DepthSum = lerp(DepthSum, 4 * PQMPAverage(Settings.PQMPContext, DepthSum), DoPerQuad);
	DepthSquaredSum = lerp(DepthSquaredSum, 4 * PQMPAverage(Settings.PQMPContext, DepthSquaredSum), DoPerQuad);
	SumWeight = lerp(SumWeight, 4 * PQMPAverage(Settings.PQMPContext, SumWeight), DoPerQuad);
	UVOffsetSum = lerp(UVOffsetSum, 4 * PQMPAverage(Settings.PQMPContext, UVOffsetSum), DoPerQuad);
	UVOffsetSquareSum = lerp(UVOffsetSquareSum, 4 * PQMPAverage(Settings.PQMPContext, UVOffsetSquareSum), DoPerQuad);
	UVCrossSum = lerp(UVCrossSum, 4 * PQMPAverage(Settings.PQMPContext, UVCrossSum), DoPerQuad);
#endif

	if (SumWeight == 0)
	{
		// If no occluder found, early return as if this pixel was unshadowed.
		return PCSS_DEBUG_EARLY_RETURN ? 0.8 : 1;
	}
	else if (SumWeight >= lerp(PCSS_SEARCH_SAMPLES, 4 * PCSS_SEARCH_SAMPLES - 0.5, DoPerQuad))
	{
		// If every occluder search sample have found an occluder, then early return as if this pixel was totaly shadowed.
		return PCSS_DEBUG_EARLY_RETURN ? 0.2 : 0;
	}

	float NormalizeFactor = (1 / SumWeight);
	float2 OccluderUVGradient = UVOffsetSum * NormalizeFactor;
	float2x2 PCFUVMatrix = PerPixelRotationMatrix;

#if 0 // Signal analysis:
	// This shows the direction of the UV offset average of the occluders.
	if (Settings.DebugScreenUV.x > 0.7 && 1) return PCSSDebugUVDir(OccluderUVGradient);

	// This shows the length of the UV offset average of the occluders.
	if (Settings.DebugScreenUV.x > 0.5 && 0) return length(OccluderUVGradient);

	// This shows the number of occluders.
	if (Settings.DebugScreenUV.x > 0.5 && 0) return SumWeight / 3;
#endif

	float DepthAvg = DepthSum * NormalizeFactor;
	float DepthVariance = NormalizeFactor * DepthSquaredSum - DepthAvg * DepthAvg;
	float DepthStandardDeviation = sqrt(DepthVariance);
	float AverageOccluderDistance = Settings.SceneDepth - DepthAvg;
#if SPOT_LIGHT_PCSS
	float Penumbra = Settings.ProjectedSourceRadius * AverageOccluderDistance / DepthAvg;
#else
	float Penumbra = Settings.TanLightSourceAngle * AverageOccluderDistance;
#endif
	Penumbra = min(Penumbra, Settings.MaxKernelSize);

	float RawFilterRadius = RandomFilterScale * Penumbra;
	float FilterRadius = max(PCFMinFilterSize, RawFilterRadius);
	float TanDirectionalLightAngle = FilterRadius / (RandomFilterScale * AverageOccluderDistance);

#if PCSS_ANTI_ALIASING_METHOD == 1 // Experiement 4: do elliptical PCF kernel according to the occluder UV gradient.
	{
		float ElepticalFading = PCFMinFilterSize / FilterRadius;
		float ElepticalFactor = length(OccluderUVGradient) * SumWeight * ElepticalFading;
		float2 GrandAxisDir = normalize(float2(OccluderUVGradient.y, -OccluderUVGradient.x));
		float2x2 ElepticalProjectionMatrix = GenerateDirectionalScale2x2Matrix(GrandAxisDir, ElepticalFactor);

		PCFUVMatrix = mul(ElepticalProjectionMatrix, PCFUVMatrix);
	}
#elif PCSS_ANTI_ALIASING_METHOD == 2
	{
		// Minimal number of times the longest egeinst vector should be from the shorter one.
		const float EigenThreshold = 1.3;

		// Number of occluders requiered to consider the covriance matrix valid.
		const float MinimumCovaranceOccluders = 2;

		// Compute covariance matrix:
		//		a b
		//		b c
		//
		// With:
		//	a == CovarianceMatrixDiagonal.x
		//	b == CovarianceMatrixNonDiagonal
		//	c == CovarianceMatrixDiagonal.y
		float CovarianceMatrixNonDiagonal = SumWeight * UVCrossSum - OccluderUVGradient.x * OccluderUVGradient.y;
		float2 CovarianceMatrixDiagonal = float2(
			SumWeight * UVOffsetSquareSum.x - OccluderUVGradient.x * OccluderUVGradient.x,
			SumWeight * UVOffsetSquareSum.y - OccluderUVGradient.y * OccluderUVGradient.y);

		// Compute covariance matrix's Eigen values.
		float CovarianceMatrixTrace = CovarianceMatrixDiagonal.x + CovarianceMatrixDiagonal.y;
		float T = sqrt(CovarianceMatrixTrace * CovarianceMatrixTrace -
			4 * (CovarianceMatrixDiagonal.x * CovarianceMatrixDiagonal.y - CovarianceMatrixNonDiagonal * CovarianceMatrixNonDiagonal));
		float2 EigenValues = 0.5 * (CovarianceMatrixTrace + float2(T, -T));

		// Choose the longest Egein vector.
		float MaxEigenValue = max(EigenValues.x, EigenValues.y);
		float MinEigenValue = min(EigenValues.x, EigenValues.y);
		//float2 EigenVector = normalize(float2(CovarianceMatrixNonDiagonal, MaxEigenValue - CovarianceMatrixDiagonal.x));
		float2 LongestEigenVector = normalize(float2(MaxEigenValue - CovarianceMatrixDiagonal.y, CovarianceMatrixNonDiagonal));
		float LongestEigenVectorFactor = sqrt(MaxEigenValue / MinEigenValue) - EigenThreshold;
		float CovarianceBasedEllipticalFactor = 8 * LongestEigenVectorFactor;
		
		// Linear interpolator between the average occluder UV offset method (0) to the covariance method (1).
		float CovarianceFade = saturate((SumWeight >= MinimumCovaranceOccluders ? 1 : 0) * LongestEigenVectorFactor);
		
		// Heuristic: Force to use the average occluder UV offset method if the length of average of the offset is high.
		CovarianceFade *= saturate(1 - 5 * (length(OccluderUVGradient) - 0.5));

		// Ellipse's major axis computed from the average occluder UV offset. 
		float2 AverageBasedMajorAxis = normalize(float2(OccluderUVGradient.y, -OccluderUVGradient.x));
		float AverageBasedEllipticalFactor = length(OccluderUVGradient) * SumWeight;

		// Ellipitical fade based on how many time the filter size.
		float EllipticalFade = PCFMinFilterSize / FilterRadius;

		// Final ellipse's major axis and scale.
		float2 EllipseMajorAxis = normalize(lerp(AverageBasedMajorAxis, LongestEigenVector, CovarianceFade));
		float MaxEllipticalFactor = 6 - 3 * CovarianceFade;
		float EllipticalFactor = min(lerp(AverageBasedEllipticalFactor, CovarianceBasedEllipticalFactor, CovarianceFade), MaxEllipticalFactor) * EllipticalFade;

		// Heuristic: reduce the ellipse shapse in corners of light.
		//float ShadowCornerMultiplier = 1 - pow(saturate(30 * (length(OccluderUVGradient) - 0.45)) * abs(dot(AverageBasedMajorAxis, LongestEigenVector)), 3);
		//EllipticalFactor *= ShadowCornerMultiplier;

		// Projection matrix that transform the disk shaped kernel to an ellipse.
		float2x2 ElepticalProjectionMatrix = GenerateDirectionalScale2x2Matrix(EllipseMajorAxis, EllipticalFactor);

		// Hacks the PCF's per pixel UV rotation matrix.
		PCFUVMatrix = mul(ElepticalProjectionMatrix, PCFUVMatrix);

#if 0 // Final major axis analysis.
		if (Settings.DebugScreenUV.x > 0.5) return PCSSDebugUVDir(EllipseMajorAxis);

#elif 0 // Final elliptical factor
		if (Settings.DebugScreenUV.x > 0.5) return EllipticalFactor * 0.1;

#elif 0 // Covarance fade factor
		if (Settings.DebugScreenUV.x > 0.5) return CovarianceFade;

#elif 0 // Not enough occluder samples.
		if (Settings.DebugScreenUV.x > 0.5) return SumWeight < 3 ? 1 : 0;

#elif 0
		if (Settings.DebugScreenUV.x > 0.5) return ShadowCornerMultiplier;

#elif 0 // Egein vector 
		if (Settings.DebugScreenUV.x > 0.5) return sqrt(MaxEigenValue / MinEigenValue) - 1;

#elif 0
		if (Settings.DebugScreenUV.x > 0.5) return (length(OccluderUVGradient) - 0.7) * 5;

#elif 0 // Signal analysis.
		if (Settings.DebugScreenUV.x > 0.7 && 1) return EllipticalFade;
		if (Settings.DebugScreenUV.x > 0.5 && 1) return EllipticalFactor;
		if (Settings.DebugScreenUV.x > 0.3 && 1) return CovarianceFade;

#elif 0 // Major axis transition analysis.
		if (Settings.DebugScreenUV.x > 0.7) return PCSSDebugUVDir(AverageBasedMajorAxis);
		if (Settings.DebugScreenUV.x > 0.5) return PCSSDebugUVDir(LongestEigenVector);
		if (Settings.DebugScreenUV.x > 0.3) return CovarianceFade;
#endif
	}
#endif

#if PCSS_PCF_EXPERIMENT == 1 // -> Shadow occlusion sum.
	// UV area covered by one sample one the disk on the average depth. 
	const float NormalizedSampleArea = 1 / float(PCSS_SAMPLES);
	
	// PCF loop.
	float ShadowOcclusionSum = 0;
	for (int j = 0; j < PCSS_SAMPLES; j++)
	{
		float2 E = SobolIndex(SobolRandom, j << 3, PCSS_SAMPLE_BITS + 3) / float(0x10000);
		float2 PCFSample = UniformSampleDiskConcentricApprox(E);

		float OriginalUVLength = length(PCFSample);
		float2 SampleUVOffset = mul(PCFUVMatrix, PCFSample) * FilterRadius;// *pow(OriginalUVLength, 0.9);
		float2 SampleUV = ShadowPosition + SampleUVOffset * Settings.ShadowTileOffsetAndSize.zw;
		float SampleDepth = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, SampleUV, 0).r;
		float ShadowDepthCompare = Settings.SceneDepth - SampleDepth;

		float SampleDepthBias = length(SampleUVOffset) / TanLightSourceAngle;
		//float SampleDepthBias = max(dot(DepthBiasDotFactors, SampleUVOffset), 0);

		if (ShadowDepthCompare > SampleDepthBias)
		{
			float SampleSizeMultiplier = AverageOccluderDistance / min(ShadowDepthCompare, AverageOccluderDistance);
			ShadowOcclusionSum += NormalizedSampleArea * SampleSizeMultiplier * SampleSizeMultiplier;
		}
	}
	ShadowOcclusionSum = lerp(ShadowOcclusionSum, PQMPAverage(Settings.PQMPContext, ShadowOcclusionSum), DoPerQuad * 0.5);
	float Visibility = max(1.0 - ShadowOcclusionSum, 0.0);

#else
	// PCF loop.
	float VisibleLightAccumulation = 0;
	for (int j = 0; j < PCSS_SAMPLES; j++)
	{
		float2 E = SobolIndex(SobolRandom, j << 3, PCSS_SAMPLE_BITS + 3) / float(0x10000);
		float2 PCFSample = UniformSampleDiskConcentricApprox(E);
		float2 SampleUVOffset = mul(PCFUVMatrix, PCFSample) * FilterRadius;
		float2 SampleUV = ShadowPosition + SampleUVOffset * Settings.ShadowTileOffsetAndSize.zw;
		float SampleDepth = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, SampleUV, 0).r;

		float SampleDepthBias = max(dot(DepthBiasDotFactors, SampleUVOffset), 0);

		VisibleLightAccumulation += saturate((SampleDepth - Settings.SceneDepth + SampleDepthBias) * Settings.TransitionScale + 1);
	}

	float Visibility = VisibleLightAccumulation / float(PCSS_SAMPLES);

#endif // PCSS_PCF_EXPERIMENT == 0

#if PCSS_ENABLE_POST_PCF_SHARPENING // Sharpen the PCF result.
	{
		// Average distance between samles in UV space.
		const float AverageSampleDistance = sqrt(1 / (float(PCSS_SAMPLES) * PI));

		// Maximum factor of sharpness to introduce.
		const float MaxSharpnessFactor = 4;

		// Filter to avoid applying the sharpening filter on high frequency shadows.
		float SharpenessFading = saturate(1.5 * (length(OccluderUVGradient) - AverageSampleDistance));

		// Factor introducing the sharpness.
		float FinalSharpenessFactor = lerp(1, clamp(PCFMinFilterSize / RawFilterRadius, 1, MaxSharpnessFactor), SharpenessFading);

		// Apply the sharpness onto the visibility.
		Visibility = saturate(FinalSharpenessFactor * (Visibility - 0.5) + 0.5);

#if 0
		if (Settings.DebugScreenUV.x > 0.8) return SharpeningFade;
		if (Settings.DebugScreenUV.x > 0.5) return (SharpeningFactor - 1) * 0.25;
#endif
	}
#endif //PCSS_ENABLE_POST_PCF_SHARPENING

#if PCSS_SHARE_PER_PIXEL_QUAD >= 2 // Share PCF computation.
	Visibility = lerp(Visibility, PQMPAverage(Settings.PQMPContext, Visibility), DoPerQuad * 0.5);
#endif

	return Visibility;
}
