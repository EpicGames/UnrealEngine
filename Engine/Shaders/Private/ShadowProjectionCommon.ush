// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowProjectionCommon.usf: Contains functions that uniformly filter a depth buffer.
=============================================================================*/

Texture2D ShadowDepthTexture;
SamplerState ShadowDepthTextureSampler;
// xy:unused, z:SoftTransitionScale
float3 SoftTransitionScale;
// xy:ShadowTexelSize.xy, zw:1/ShadowTexelSize.xy
float4 ShadowBufferSize;

/** Cube map texture. */
TextureCube ShadowDepthCubeTexture;

// This is required on GLSL based languages as the standard does NOT allow having a texture being used with both a Comparison and regular SamplerState
#define USE_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE (OPENGL_PROFILE || COMPILER_VULKAN || PLATFORM_NEEDS_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE)
#if USE_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE
TextureCube ShadowDepthCubeTexture2;
#endif

/** Sampler state used for hardware PCF. */
SamplerComparisonState ShadowDepthCubeTextureSampler;

/** View projection matrices that were used in the shadow depth pass for each cube face. */
float4x4 ShadowViewProjectionMatrices[6];

float InvShadowmapResolution;

// @param SideVector needs to be scale with InvShadowmapResolution
// @param UpVector needs to be scale with InvShadowmapResolution
float CubePCF3x3Quarter(float3 NormalizedLightVector, float3 SideVector, float3 UpVector, float CompareDistance, float Bias = 0)
{
	float3 Sample00Coordinate = NormalizedLightVector + SideVector * 0.5f + UpVector * 0.5f;
	float3 Sample01Coordinate = NormalizedLightVector + SideVector * 0.5f + UpVector * 1.5f;
	float3 Sample02Coordinate = NormalizedLightVector + SideVector * 0.5f + UpVector * 2.5f;
	float3 Sample10Coordinate = NormalizedLightVector + SideVector * 1.5f + UpVector * 0.5f;
	float3 Sample11Coordinate = NormalizedLightVector + SideVector * 1.5f + UpVector * 1.5f;
	float3 Sample12Coordinate = NormalizedLightVector + SideVector * 1.5f + UpVector * 2.5f;
	float3 Sample20Coordinate = NormalizedLightVector + SideVector * 2.5f + UpVector * 0.5f;
	float3 Sample21Coordinate = NormalizedLightVector + SideVector * 2.5f + UpVector * 1.5f;
	float3 Sample22Coordinate = NormalizedLightVector + SideVector * 2.5f + UpVector * 2.5f;

	// Lookup and combine 9 hardware PCF lookups
	float3 ShadowResults0;
	ShadowResults0.x = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample00Coordinate, CompareDistance + Bias * length(float2(0.5f, 0.5f)));
	ShadowResults0.y = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample01Coordinate, CompareDistance + Bias * length(float2(0.5f, 1.5f)));
	ShadowResults0.z = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample02Coordinate, CompareDistance + Bias * length(float2(0.5f, 2.5f)));

	float3 ShadowResults1;
	ShadowResults1.x = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample10Coordinate, CompareDistance + Bias * length(float2(1.5f, 0.5f)));
	ShadowResults1.y = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample11Coordinate, CompareDistance + Bias * length(float2(1.5f, 1.5f)));
	ShadowResults1.z = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample12Coordinate, CompareDistance + Bias * length(float2(1.5f, 2.5f)));

	float3 ShadowResults2;
	ShadowResults2.x = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample20Coordinate, CompareDistance + Bias * length(float2(2.5f, 0.5f)));
	ShadowResults2.y = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample21Coordinate, CompareDistance + Bias * length(float2(2.5f, 1.5f)));
	ShadowResults2.z = ShadowDepthCubeTexture.SampleCmpLevelZero(ShadowDepthCubeTextureSampler, Sample22Coordinate, CompareDistance + Bias * length(float2(2.5f, 2.5f)));

	return dot(ShadowResults0 + ShadowResults1 + ShadowResults2, .1111111f);
}

static const float2 DiscSamples5[]=
{ // 5 random points in disc with radius 2.500000
	float2(0.000000, 2.500000),
	float2(2.377641, 0.772542),
	float2(1.469463, -2.022543),
	float2(-1.469463, -2.022542),
	float2(-2.377641, 0.772543),
};

static const float2 DiscSamples12[]=
{ // 12 random points in disc with radius 2.500000
	float2(0.000000, 2.500000),
	float2(1.767767, 1.767767),
	float2(2.500000, -0.000000),
	float2(1.767767, -1.767767),
	float2(-0.000000, -2.500000),
	float2(-1.767767, -1.767767),
	float2(-2.500000, 0.000000),
	float2(-1.767766, 1.767768),
	float2(-1.006119, -0.396207),
	float2(1.000015, 0.427335),
	float2(0.416807, -1.006577),
	float2(-0.408872, 1.024430),
};

static const float2 DiscSamples29[]=
{ // 29 random points in disc with radius 2.500000
	float2(0.000000, 2.500000),
	float2(1.016842, 2.283864),
	float2(1.857862, 1.672826),
	float2(2.377641, 0.772542),
	float2(2.486305, -0.261321),
	float2(2.165063, -1.250000),
	float2(1.469463, -2.022543),
	float2(0.519779, -2.445369),
	float2(-0.519779, -2.445369),
	float2(-1.469463, -2.022542),
	float2(-2.165064, -1.250000),
	float2(-2.486305, -0.261321),
	float2(-2.377641, 0.772543),
	float2(-1.857862, 1.672827),
	float2(-1.016841, 2.283864),
	float2(0.091021, -0.642186),
	float2(0.698035, 0.100940),
	float2(0.959731, -1.169393),
	float2(-1.053880, 1.180380),
	float2(-1.479156, -0.606937),
	float2(-0.839488, -1.320002),
	float2(1.438566, 0.705359),
	float2(0.067064, -1.605197),
	float2(0.728706, 1.344722),
	float2(1.521424, -0.380184),
	float2(-0.199515, 1.590091),
	float2(-1.524323, 0.364010),
	float2(-0.692694, -0.086749),
	float2(-0.082476, 0.654088),
};

void CalcCubemapVectorAndFaceIndex(float3 NormalizedLightVector, float ShadowFilterRadius, out float3 SideVector, out float3 UpVector, out int CubeFaceIndex)
{
	SideVector = normalize(cross(NormalizedLightVector, float3(0, 0, 1)));
	UpVector = cross(SideVector, NormalizedLightVector);

	SideVector *= InvShadowmapResolution * ShadowFilterRadius;
	UpVector *= InvShadowmapResolution * ShadowFilterRadius;

	// TODO: this should use GCN intrinsic when available

	// Figure out which cube face we're sampling from
	float3 AbsLightVector = abs(NormalizedLightVector);
	float MaxCoordinate = max(AbsLightVector.x, max(AbsLightVector.y, AbsLightVector.z));

	if (MaxCoordinate == AbsLightVector.x)
	{
		CubeFaceIndex = AbsLightVector.x == NormalizedLightVector.x ? 0 : 1;
	}
	else if (MaxCoordinate == AbsLightVector.y)
	{
		CubeFaceIndex = AbsLightVector.y == NormalizedLightVector.y ? 2 : 3;
	}
	else
	{
		CubeFaceIndex = AbsLightVector.z == NormalizedLightVector.z ? 4 : 5;
	}
}

/** Computes shadowing for a given world position from a cubemap shadowmap used on a point light. */
float CubemapHardwarePCF(
	TextureCube InShadowDepthCubeTexture, SamplerComparisonState InShadowDepthCubeTextureSampler, float4x4 InShadowViewProjectionMatrices[6], float InInvShadowmapResolution,
	float3 WorldPosition, float3 LightPosition, float LightInvRadius, float DepthBias, float SlopeDepthBias, float MaxSlopeDepthBias)
{
	float Shadow = 1;

	float3 WorldSampleToLightVec = LightPosition - WorldPosition.xyz;
	float Distance = length(WorldSampleToLightVec);

	BRANCH
	// Skip pixels outside of the light's influence
	if (Distance * LightInvRadius < 1.0f)
	{
		float3 NormalizedLightVector = WorldSampleToLightVec / Distance;
		float3 SideVector = normalize(cross(NormalizedLightVector, float3(0, 0, 1)));
		float3 UpVector = cross(SideVector, NormalizedLightVector);

		SideVector *= InvShadowmapResolution;
		UpVector *= InvShadowmapResolution;

		// Figure out which cube face we're sampling from
		float3 AbsLightVector = abs(WorldSampleToLightVec);
		float MaxCoordinate = max(AbsLightVector.x, max(AbsLightVector.y, AbsLightVector.z));

		int CubeFaceIndex = 0;
		if (MaxCoordinate == AbsLightVector.x)
		{
			CubeFaceIndex = AbsLightVector.x == WorldSampleToLightVec.x ? 0 : 1;
		}
		else if (MaxCoordinate == AbsLightVector.y)
		{
			CubeFaceIndex = AbsLightVector.y == WorldSampleToLightVec.y ? 2 : 3;
		}
		else
		{
			CubeFaceIndex = AbsLightVector.z == WorldSampleToLightVec.z ? 4 : 5;
		}

		// Transform the Light-relative position into shadow space (The light shadow view is pre-translated wrt the main view)
		float4 ShadowPosition = mul(float4(-WorldSampleToLightVec, 1), InShadowViewProjectionMatrices[CubeFaceIndex]);

		// Calculate the Z buffer value that would have been stored for this position in the shadow map
		float CompareDistance = ShadowPosition.z / ShadowPosition.w;
		float ShadowDepthBias = - (DepthBias + SlopeDepthBias) / ShadowPosition.w;

		Shadow = 0;

#if SHADOW_QUALITY <= 2

		Shadow = InShadowDepthCubeTexture.SampleCmpLevelZero(InShadowDepthCubeTextureSampler, WorldSampleToLightVec, CompareDistance - ShadowDepthBias);

#elif SHADOW_QUALITY == 3
		UNROLL for(int i = 0; i < 5; ++i)
		{
			float3 SamplePos = NormalizedLightVector + SideVector * DiscSamples5[i].x + UpVector * DiscSamples5[i].y;
			Shadow += InShadowDepthCubeTexture.SampleCmpLevelZero(
				InShadowDepthCubeTextureSampler,
				SamplePos, 
				CompareDistance - ShadowDepthBias * length(DiscSamples5[i]));
		}
		Shadow /= 5;

#elif SHADOW_QUALITY == 4

		UNROLL for(int i = 0; i < 12; ++i)
		{
			float3 SamplePos = NormalizedLightVector + SideVector * DiscSamples12[i].x + UpVector * DiscSamples12[i].y;
			Shadow += InShadowDepthCubeTexture.SampleCmpLevelZero(
				InShadowDepthCubeTextureSampler,
				SamplePos, 
				CompareDistance - ShadowDepthBias * length(DiscSamples12[i]));
		}
		Shadow /= 12;

#else // SHADOW_QUALITY

		UNROLL for(int i = 0; i < 29; ++i)
		{
			float3 SamplePos = NormalizedLightVector + SideVector * DiscSamples29[i].x + UpVector * DiscSamples29[i].y;
			Shadow += InShadowDepthCubeTexture.SampleCmpLevelZero(
				InShadowDepthCubeTextureSampler,
				SamplePos, 
				CompareDistance - ShadowDepthBias * length(DiscSamples29[i]));
		}
		Shadow /= 29;

#endif // SHADOW_QUALITY
		
		/*// 4 * 9 samples (former non randomized sample pattern, more samples, less ALU, slower)
		{
			float4 ShadowResults;
			// TODO CubePCF3x3Quarter should use the In parameters similarly to InShadowDepthCubeTexture
			ShadowResults.x = CubePCF3x3Quarter(NormalizedLightVector, - SideVector, - UpVector, CompareDistance, ShadowDepthBias);
			ShadowResults.y = CubePCF3x3Quarter(NormalizedLightVector, - SideVector,   UpVector, CompareDistance, ShadowDepthBias);
			ShadowResults.z = CubePCF3x3Quarter(NormalizedLightVector,   SideVector, - UpVector, CompareDistance, ShadowDepthBias);
			ShadowResults.w = CubePCF3x3Quarter(NormalizedLightVector,   SideVector,   UpVector, CompareDistance, ShadowDepthBias);
				
			Shadow = dot(ShadowResults, .25f);
		}*/
	}

	// ReverseZ so flip the result
	return 1.0f - Shadow;
}

float CubemapHardwarePCF(
	float3 WorldPosition, float3 LightPosition, float LightInvRadius, float DepthBias, float SlopeDepthBias, float MaxSlopeDepthBias)
{
	return CubemapHardwarePCF(ShadowDepthCubeTexture, ShadowDepthCubeTextureSampler, ShadowViewProjectionMatrices, InvShadowmapResolution,
		WorldPosition, LightPosition, LightInvRadius, DepthBias, SlopeDepthBias, MaxSlopeDepthBias);
}

#ifndef NO_TRANSLUCENCY_AVAILABLE

/** Textures and parameters needed for Fourier opacity maps. */
#define TranslucencyShadowTransmission0 TranslucentSelfShadow.Transmission0
#define TranslucencyShadowTransmission1 TranslucentSelfShadow.Transmission1

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define TranslucencyShadowTransmission0Sampler View.SharedBilinearClampedSampler
	#define TranslucencyShadowTransmission1Sampler View.SharedBilinearClampedSampler
#else
	#define TranslucencyShadowTransmission0Sampler TranslucentSelfShadow.Transmission0Sampler
	#define TranslucencyShadowTransmission1Sampler TranslucentSelfShadow.Transmission1Sampler
#endif

/** These have been tweaked to minimize ringing while not losing too much high frequency in the captured opacity function. */
static const float RingingSuppressionFactor = 1;
static const float NumTermsForRingingSuppression = 8;

// Creates a scale for a given frequency that minimizes ringing inherent with a Fourier basis
// This is done by scaling down the contribution of the higher frequency components
// Using a macro to support any size vector
#define RingingSuppressionScale(k) (exp(-RingingSuppressionFactor * Square(k / NumTermsForRingingSuppression)))

float CalculateTranslucencyShadowingDensity(float2 ShadowUV, float ShadingDepth)
{
// Needs to match the corresponding define in TranslucentShadowDepthShaders.usf
#define USE_FOURIER_OPACITY_MAP 1
#if USE_FOURIER_OPACITY_MAP

	// Fourier opacity shadow map
	// Fetch the accumulated Fourier coefficients generated by rendering all translucent casters
	float4 CosCoefficients0 = Texture2DSampleLevel(TranslucencyShadowTransmission0, TranslucencyShadowTransmission0Sampler, ShadowUV, 0);
	float4 SinCoefficients0 = Texture2DSampleLevel(TranslucencyShadowTransmission1, TranslucencyShadowTransmission1Sampler, ShadowUV, 0);

	float3 FrequencyScales0 = 2.0 * PI * float3(1, 2, 3);
	float3 CoefficientScales0 = RingingSuppressionScale(float3(1, 2, 3)) / FrequencyScales0;

	// Calculate the sin and cos wave scales for each frequency based on the depth of the point being shaded
	float3 ShadingSinCoefficient0;
	float3 ShadingCosCoefficient0;
	sincos(FrequencyScales0 * ShadingDepth, ShadingSinCoefficient0, ShadingCosCoefficient0);
	ShadingCosCoefficient0 = 1 - ShadingCosCoefficient0;
	
	// Dot the two sets of Fourier coefficients to calculate the density from translucent casters at the shading point along the ray to the light
	float FinalDensity = (CosCoefficients0.x * ShadingDepth / 2.0) + dot(ShadingSinCoefficient0 * CosCoefficients0.yzw * CoefficientScales0, 1) + dot(ShadingCosCoefficient0 * SinCoefficients0.yzw * CoefficientScales0, 1);
	return FinalDensity;

#else

	// Opacity shadow map
	float LayerSize = 1.0f / 15.0f;

	float4 LayerDepths0 = float4(0, 1, 2, 3) * LayerSize;
	float4 LayerDepths1 = float4(4, 5, 6, 7) * LayerSize;
	float4 LayerDepths2 = float4(8, 9, 10, 11) * LayerSize;
	float4 LayerDepths3 = float4(12, 13, 14, 15) * LayerSize;

	float4 Densities0 = Texture2DSampleLevel(TranslucencyShadowTransmission0, TranslucencyShadowTransmission0Sampler, ShadowUV, 0);
	float4 Densities1 = Texture2DSampleLevel(TranslucencyShadowTransmission1, TranslucencyShadowTransmission1Sampler, ShadowUV, 0);
	float4 Densities2 = Texture2DSampleLevel(TranslucencyShadowTransmission2, TranslucencyShadowTransmission2Sampler, ShadowUV, 0);
	float4 Densities3 = Texture2DSampleLevel(TranslucencyShadowTransmission3, TranslucencyShadowTransmission3Sampler, ShadowUV, 0);

	// Setup lerp fractions between layers
	float4 LayerFractions0 = (ShadingDepth - LayerDepths0) / LayerSize;
	float4 LayerFractions1 = (ShadingDepth - LayerDepths1) / LayerSize;
	float4 LayerFractions2 = (ShadingDepth - LayerDepths2) / LayerSize;
	float4 LayerFractions3 = (ShadingDepth - LayerDepths3) / LayerSize;

	// Setup a mask where the active layer has a value of 1, all other layers have 0
	// LayerMask0.x has no lower bound, and LayerMask3.w has no upper bound, so that pixels outside the range of the opacity shadow map are handled
	float4 LayerMask0 = (LayerFractions0 < 1) * float4(1, LayerFractions0.yzw >= 0);
	float4 LayerMask1 = (LayerFractions1 < 1) * (LayerFractions1 >= 0);
	float4 LayerMask2 = (LayerFractions2 < 1) * (LayerFractions2 >= 0);
	float4 LayerMask3 = float4(LayerFractions3.xyz < 1, 1) * (LayerFractions3 >= 0);

	// Lerp between the nearest 2 layer's transmissions, using LayerMask to select layer attributes
	float FinalDensity0 = lerp(dot(LayerMask0, float4(0, Densities0.xyz)), dot(LayerMask0, Densities0), saturate(dot(LayerMask0, LayerFractions0)));
	float FinalDensity1 = lerp(dot(LayerMask1, float4(Densities0.w, Densities1.xyz)), dot(LayerMask1, Densities1), saturate(dot(LayerMask1, LayerFractions1)));
	float FinalDensity2 = lerp(dot(LayerMask2, float4(Densities1.w, Densities2.xyz)), dot(LayerMask2, Densities2), saturate(dot(LayerMask2, LayerFractions2)));
	float FinalDensity3 = lerp(dot(LayerMask3, float4(Densities2.w, Densities3.xyz)), dot(LayerMask3, Densities3), saturate(dot(LayerMask3, LayerFractions3)));

	float FinalDensity = FinalDensity0 + FinalDensity1 + FinalDensity2 + FinalDensity3;

	return FinalDensity;

#endif
}

float CalculateTranslucencyShadowing(float2 ShadowUV, float ShadingDepth)
{
	float ShadowDensity = CalculateTranslucencyShadowingDensity(ShadowUV, ShadingDepth);
	float FinalTransmission = saturate(exp(-ShadowDensity));
	return FinalTransmission;
}

#endif // NO_TRANSLUCENCY_AVAILABLE


