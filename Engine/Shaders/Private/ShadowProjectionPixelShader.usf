// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowProjectionPixelShader.usf: Pixel shader for projecting a shadow depth buffer onto the scene.
=============================================================================*/

#ifndef USE_FADE_PLANE
	#define USE_FADE_PLANE 0
#endif

#ifndef SHADOW_QUALITY
	#define SHADOW_QUALITY 6
#endif

#ifndef APPLY_TRANSLUCENCY_SHADOWS
	#define APPLY_TRANSLUCENCY_SHADOWS 0
#endif

#ifndef USE_PCSS
	#define USE_PCSS 0
#endif

#ifndef SPOT_LIGHT_PCSS
#define SPOT_LIGHT_PCSS 0
#endif

#include "HairStrands/HairStrandsVisibilityCommonStruct.ush"
#include "Common.ush"

// Needs to bind MobileBasePass UB since the modulated shadow is in mobile base pass. 
#if MODULATED_SHADOWS
	#define MobileSceneTextures MobileBasePass.SceneTextures
#endif
#if SHADING_PATH_MOBILE
#include "MobileLightingCommon.ush"
#endif

#include "ShadowProjectionCommon.ush"
#include "ShadowFilteringCommon.ush"
#if USE_PCSS
#include "ShadowPercentageCloserFiltering.ush"
#endif
#include "DeferredShadingCommon.ush"
#include "DynamicLightingCommon.ush"
#include "ToonProfileCommon.ush"

float ShadowFadeFraction;
float ShadowSharpen;
float4 LightPositionAndInvRadius;

float PerObjectShadowFadeStart;
float InvPerObjectShadowFadeLength;

#if USE_TRANSMISSION
	#include "TransmissionThickness.ush"
#else
	float4x4 ScreenToShadowMatrix;
	// .x:DepthBias, .y:SlopeDepthBias, .z:ReceiverBias, .w: MaxSubjectZ - MinSubjectZ
	float4 ProjectionDepthBiasParameters;
#endif

	float4 LightPositionOrDirection;
	float ShadowReceiverBias;

#if USE_FADE_PLANE || SUBPIXEL_SHADOW
	float FadePlaneOffset;
	float InvFadePlaneLength;
	uint bCascadeUseFadePlane;
#endif

#if USE_PCSS
	// PCSS specific parameters.
	//	- x: tan(0.5 * Directional Light Angle) in shadow projection space;
	//  - y: Max filter size in shadow tile UV space.
	float4 PCSSParameters;
#endif

float4 ModulatedShadowColor;
float4 ShadowTileOffsetAndSize;

float3 ModulatedShadowBlendOp(float3 Source)
{
	half4 Dest = half4(0, 0, 0, 0);
	return Source.rgb*Dest.rgb;
}

// PCF falloff is overly blurry, apply to get
// a more reasonable look artistically. Should not be applied to
// other passes that target pbr (e.g., raytracing and denoising).
float ApplyPCFOverBlurCorrection(float Shadow)
{
	return Square(Shadow);
}

#if SUBPIXEL_SHADOW
#include "HairStrands/HairStrandsVisibilityCommon.ush"
float2 ShadowNearAndFarDepth;
#endif

#include "Substrate/Substrate.ush"

/**
* Entry point for uniform manual PCF that supports lights using normal shadows.
*/
// Do not force early_fragment_tests on mobile, as it does not work with depth fetch on MALI
#if (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)
EARLYDEPTHSTENCIL
#endif
void Main(
	in float4 SVPos : SV_POSITION,
	in FStereoPSInput StereoInput,
	out float4 OutColor : SV_Target0
	)
{
	#if USE_FADE_PLANE
	const bool bUseFadePlane = true;
	#endif
	
	StereoSetupPS(StereoInput);

	const FPQMPContext PQMPContext = PQMPInit(SVPos.xy);
	float2 ScreenUV = float2(SVPos.xy * ResolvedView.BufferSizeAndInvSize.zw);
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
	float SceneW = CalcSceneDepth(ScreenUV, GetEyeIndex(StereoInput));
#else
	float SceneW = CalcSceneDepth(ScreenUV);
#endif

	bool bIsSubsurfaceCompatible = true;
	const uint3 PixelCoord = uint3(floor(SVPos.xy), 0);
#if SUBPIXEL_SHADOW
	const bool bUseFadePlane = bCascadeUseFadePlane > 0;
	SceneW = ConvertFromDeviceZ(HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x);
	bIsSubsurfaceCompatible = false;
#endif
	float4 ScreenPosition = float4(GetScreenPositionForProjectionType(((ScreenUV.xy - ResolvedView.ScreenPositionScaleBias.wz) / ResolvedView.ScreenPositionScaleBias.xy), SceneW), SceneW, 1);
	
	float4 ShadowPosition = float4(0, 0, 0, 0);
	
#if MOBILE_MULTI_VIEW && SHADING_PATH_MOBILE
	ShadowPosition = mul(ScreenPosition, ResolvedView.MobileMultiviewShadowTransform);
	ShadowPosition = mul(ShadowPosition, ScreenToShadowMatrix);
#else
	ShadowPosition = mul(ScreenPosition, ScreenToShadowMatrix);
#endif
	float3 TranslateWorldPosition = mul(ScreenPosition, ResolvedView.ScreenToTranslatedWorld).xyz;

	float ShadowZ = 1 - ShadowPosition.z;
	ShadowPosition.xyz /= ShadowPosition.w;

#if MODULATED_SHADOWS
	// UE-29083 : work around precision issues with ScreenToShadowMatrix on low end devices.
	ShadowPosition.xy *= ShadowTileOffsetAndSize.zw;
	ShadowPosition.xy += ShadowTileOffsetAndSize.xy;
#endif

#if USE_PCSS
	float3 ScreenPositionDDX = DDX(ScreenPosition.xyz);
	float3 ScreenPositionDDY = DDY(ScreenPosition.xyz);
	float4 ShadowPositionDDX = mul(float4(ScreenPositionDDX, 0), ScreenToShadowMatrix);
	float4 ShadowPositionDDY = mul(float4(ScreenPositionDDY, 0), ScreenToShadowMatrix);
	#if SPOT_LIGHT_PCSS
		// perspective correction for derivatives, could be good enough and way cheaper to just use ddx(ScreenPosition)
		ShadowPositionDDX.xyz -= ShadowPosition.xyz * ShadowPositionDDX.w;
		ShadowPositionDDY.xyz -= ShadowPosition.xyz * ShadowPositionDDY.w;
	#endif
#endif


	// Clamp pixel depth in light space for shadowing opaque, because areas of the shadow depth buffer that weren't rendered to will have been cleared to 1
	// We want to force the shadow comparison to result in 'unshadowed' in that case, regardless of whether the pixel being shaded is in front or behind that plane
	float LightSpacePixelDepthForOpaque = min(ShadowZ, 0.99999f);
	// Must not clamp for SSS shadowing, the subsurface gradient must continue past the far plane
	float LightSpacePixelDepthForSSS = ShadowZ;

	// fade out per-object shadow before cut-off, goes from 1 at start of fade plane to 0 at the end of the fade length.
	float PerObjectDistanceFadeFraction = 1.0f - saturate((LightSpacePixelDepthForSSS - PerObjectShadowFadeStart) * InvPerObjectShadowFadeLength);

	float Shadow = 1;
	float SSSTransmittanceOrOpticalDepth = 1;
	
	float BlendFactor = 1;
	
	// For debugging
	#define UNFILTERED_SHADOW_PROJECTION 0

	#if UNFILTERED_SHADOW_PROJECTION
	{
		Shadow = LightSpacePixelDepthForOpaque < Texture2DSampleLevel(ShadowDepthTexture, ShadowDepthTextureSampler, ShadowPosition.xy, 0).r;
	}
	#elif APPLY_TRANSLUCENCY_SHADOWS
	{
		Shadow = CalculateTranslucencyShadowing(ShadowPosition.xy, ShadowZ);
	}
	#elif USE_PCSS
	{
		FPCSSSamplerSettings Settings;

		#if SPOT_LIGHT_PCSS
		{
			float CotanOuterCone = DeferredLightUniforms.SpotAngles.x * rsqrt(1. - DeferredLightUniforms.SpotAngles.x * DeferredLightUniforms.SpotAngles.x);
			float WorldLightDistance = dot(DeferredLightUniforms.Direction, GetDeferredLightTranslatedWorldPosition() - TranslateWorldPosition);
			Settings.ProjectedSourceRadius = 0.5 * DeferredLightUniforms.SourceRadius * CotanOuterCone / WorldLightDistance;
			Settings.TanLightSourceAngle = 0;
		}
		#else
		{
			Settings.ProjectedSourceRadius = 0;
			Settings.TanLightSourceAngle = PCSSParameters.x;
		}
		#endif
		Settings.ShadowDepthTexture = ShadowDepthTexture;
		Settings.ShadowDepthTextureSampler = ShadowDepthTextureSampler;
		Settings.ShadowBufferSize = ShadowBufferSize;
		Settings.ShadowTileOffsetAndSize = ShadowTileOffsetAndSize;
		Settings.SceneDepth = LightSpacePixelDepthForOpaque;
		Settings.TransitionScale = SoftTransitionScale.z;
		Settings.MaxKernelSize = PCSSParameters.y;
		Settings.SvPosition = SVPos.xy;
		Settings.PQMPContext = PQMPContext;
		Settings.DebugViewportUV = ScreenUV;
		
		Shadow = DirectionalPCSS(Settings, ShadowPosition.xy, ShadowPositionDDX.xyz, ShadowPositionDDY.xyz);
	}
	#else // !USE_PCSS
	{
		#if SHADING_PATH_DEFERRED && !FORWARD_SHADING && !SUBPIXEL_SHADOW
			// Attenuate soft transition based on the angle with the light and the shading normal (acts as a receiver bias)
			const bool bIsDirectional = LightPositionOrDirection.w == 0;
			const float3 LightDirection = bIsDirectional ? -LightPositionOrDirection.xyz : normalize(LightPositionOrDirection.xyz - TranslateWorldPosition);

		#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SVPos.xy, 0)));
			const float3 WorldNormal = TopLayerData.WorldNormal;
		#else
			FGBufferData GBufferData = GetGBufferData(ScreenUV);
			const float3 WorldNormal = GBufferData.WorldNormal;
		#endif

			const float NoL = saturate(dot(WorldNormal, LightDirection));
		#endif
		
		FPCFSamplerSettings Settings;
		
		Settings.ShadowDepthTexture = ShadowDepthTexture;
		Settings.ShadowDepthTextureSampler = ShadowDepthTextureSampler;
		Settings.ShadowBufferSize = ShadowBufferSize;
		#if SHADING_PATH_DEFERRED && !FORWARD_SHADING && !SUBPIXEL_SHADOW
		Settings.TransitionScale = SoftTransitionScale.z * lerp(ProjectionDepthBiasParameters.z, 1.0, NoL);
		#else
		Settings.TransitionScale = SoftTransitionScale.z;
		#endif
		Settings.SceneDepth = LightSpacePixelDepthForOpaque;
		Settings.bSubsurface = false;
		Settings.bTreatMaxDepthUnshadowed = false;
		Settings.DensityMulConstant = 0;
		Settings.ProjectionDepthBiasParameters = 0;
#if SHADING_PATH_MOBILE
		Shadow = MobileShadowPCF(ShadowPosition.xy, Settings);
#else
		Shadow = ManualPCF(ShadowPosition.xy, Settings);
#endif
	}
	#endif // !USE_PCSS
		
	#if USE_FADE_PLANE || SUBPIXEL_SHADOW
	if (bUseFadePlane)
	{
		// Create a blend factor which is one before and at the fade plane, and lerps to zero at the far plane.
		BlendFactor = 1.0f - saturate((SceneW - FadePlaneOffset) * InvFadePlaneLength);
	}
	#endif

	#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING && !APPLY_TRANSLUCENCY_SHADOWS && !SUBPIXEL_SHADOW

		bool bHasSubsurface				= false;
		bool bHasSubsurfaceTransmission	= false;
		bool bIsEyeOrHair 				= false;
		bool bIsToon					= false;
	#if SUBSTRATE_GBUFFER_FORMAT==1
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord.xy, uint2(ResolvedView.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstrateHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		if (SubstrateHeader.IsSubstrateMaterial())
		{
			bIsToon						= SubstrateHeader.ClosureCount == 1 && SubstrateHeader.SubstrateGetBSDFType() == SUBSTRATE_BSDF_TYPE_TOON;
			bIsEyeOrHair				= SubstrateHeader.SubstrateGetBSDFType() == SUBSTRATE_BSDF_TYPE_HAIR || SubstrateHeader.SubstrateGetBSDFType() == SUBSTRATE_BSDF_TYPE_EYE;
			bHasSubsurface				= SubstrateHeader.HasSubsurface() || bIsEyeOrHair;
			bHasSubsurfaceTransmission	= SubstrateHeader.HasSubsurface();
		}
	#else // SUBSTRATE_GBUFFER_FORMAT==0
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
		{
			bHasSubsurface = IsSubsurfaceModel(GBufferData.ShadingModelID);
			bHasSubsurfaceTransmission = GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE;
			bIsEyeOrHair = GBufferData.ShadingModelID == SHADINGMODELID_HAIR || GBufferData.ShadingModelID == SHADINGMODELID_EYE;
			bIsToon = GBufferData.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON;
		}
	#endif // SUBSTRATE_GBUFFER_FORMAT

		BRANCH
		if (bIsSubsurfaceCompatible && bHasSubsurface)
		{
			#if SUBSTRATE_ENABLED
			#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, SVPos.xy);
			const uint SSSType = SubstrateSubSurfaceHeaderGetSSSType(SSSHeader);
			#endif
			#endif // SUBSTRATE_ENABLED

			bool bApplyPCFOverBlurCorrection = true; // Set to false when the transmittance from extinction is correctly respected. Toon & TransmitttanceShadow disables that to have a physically based non squared transmittance curve.
			float Density = 1;
			if (!bIsEyeOrHair)
			{
			#if SUBSTRATE_GBUFFER_FORMAT==1
				if (SSSType == SSS_TYPE_TRANSMITTANCE_SHADOW)
				{
					bApplyPCFOverBlurCorrection = false;
					const float3 MFPcm = SubstrateSubSurfaceHeaderGetTransmittanceShadowMFPInMeters(SSSHeader) * METER_TO_CENTIMETER;
					Density = 1.0f / max(0.000001f, MFPcm.x); // Recover extinction coefficient as 1/cm from MFP in cm.
				}
				else if (SSSType != SSS_TYPE_NONE)
				{
					// Disble accurate SSS transmission if SSS type is no using Diffusion or Diffusion Profile
					bHasSubsurfaceTransmission = bHasSubsurfaceTransmission && (SSSType >= SSS_TYPE_DIFFUSION);
					Density = SubsurfaceDensityFromOpacity(SubstrateSubSurfaceHeaderGetOpacity(SSSHeader));
				}
			#else // SUBSTRATE_GBUFFER_FORMAT==0
				#if SUBSTRATE_ENABLED
				if (bIsToon)
				{
					bApplyPCFOverBlurCorrection = false;

					float ToonProfileId = 0.0f;
					float2 PatternUVs = ScreenUV;
					UnpackToonCustomData(PackRGBA8(GBufferData.CustomData), PatternUVs, ToonProfileId);

					// Density is extinction as 1/cm (not 1/m) to match with the transmittance shadow evaluation from ManualPCF below.
					FToonProfileShadowSettings ToonProfileShadowSettings = GetToonProfileShadowSettings(ToonProfileId);
					Density = ToonProfileShadowSettings.ShadowExtinctionCoefficientInInvCentimeter;
				}
				else
				#endif
				{
					Density = SubsurfaceDensityFromOpacity(GBufferData.CustomData.a);
				}
			#endif // SUBSTRATE_GBUFFER_FORMAT
			}
			
			FPCFSamplerSettings Settings;
				
			Settings.ShadowDepthTexture = ShadowDepthTexture;
			Settings.ShadowDepthTextureSampler = ShadowDepthTextureSampler;
			Settings.ShadowBufferSize = ShadowBufferSize;
			Settings.TransitionScale = SoftTransitionScale.z;
			Settings.SceneDepth = LightSpacePixelDepthForSSS + ProjectionDepthBiasParameters.x;
			Settings.bSubsurface = true;
			Settings.bTreatMaxDepthUnshadowed = false;
			Settings.DensityMulConstant = Density * ProjectionDepthBiasParameters.w;
			Settings.ProjectionDepthBiasParameters = ProjectionDepthBiasParameters.xw;

			#if USE_TRANSMISSION
			if (bHasSubsurfaceTransmission)
			{
				// This branch is only taken by SSS profiles and SSS per pixel diffusion methods.
			#if SUBSTRATE_GBUFFER_FORMAT==1
				const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SVPos.xy, 0)));
				const float3 WorldNormal = TopLayerData.WorldNormal;
				FTransmissionProfileParams TransmissionParams = GetSubstrateSubsurfaceTransmissionProfileParams(SSSType, SSSHeader);
			#else // SUBSTRATE_GBUFFER_FORMAT==1
				float3 WorldNormal = GBufferData.WorldNormal;
				FTransmissionProfileParams TransmissionParams = GetTransmissionProfileParams(ExtractSubsurfaceProfileInt(GBufferData));
			#endif // SUBSTRATE_GBUFFER_FORMAT==1

				// This branch is taken by SSSProfile and Substrate per pixel SSS.
				// => SSSTransmittanceOrOpticalDepth stores the result of CalculateEncodedOpticalDepth() evaluating OpticalDepth.
				SSSTransmittanceOrOpticalDepth = CalculateEncodedOpticalDepth(ScreenPosition.xyz, LightPositionAndInvRadius.xyz, WorldNormal, TransmissionParams, Settings);
			}
			else
			#endif
			{
				// This branch is taken by Subsurface shading model (Wrap for Substrate) or the toon shading model. 
				// => SSSTransmittanceOrOpticalDepth stores the result of ManualPCF() evaluating Transmittance.
				SSSTransmittanceOrOpticalDepth = ManualPCF(ShadowPosition.xy, Settings);
			}

			// Legacy code maintained: here we have a square applied over the transmittance curve or encoded optical depth... 
			// This could help reduce banding artefact, but since the square is not decoded in the lighting side, it is just wrong or here for an unknown reason now.
			// It affects all transmittance and encoded optical depth: substrate per pixel MFP and Subsurface Profile optical depth as well as Subsurface shading model and Toon transmittance.
			SSSTransmittanceOrOpticalDepth = bApplyPCFOverBlurCorrection ? ApplyPCFOverBlurCorrection(SSSTransmittanceOrOpticalDepth) : SSSTransmittanceOrOpticalDepth;
		}

	#endif
	
	#if !USE_PCSS
		// There is no point changing the shadow sharpen on PCSS, can directly reduce the light
		// source angle that would actually make the algorithm to run faster.
		Shadow = saturate( (Shadow - 0.5) * ShadowSharpen + 0.5 );
	#endif

	// 0 is shadowed, 1 is unshadowed
	// RETURN_COLOR not needed unless writing to SceneColor;

	// Apply PCF overblur correction for shadow when PCSS & PCF.
	#define CORRECT_SHADOW_PCF_BLUR  !UNFILTERED_SHADOW_PROJECTION && !APPLY_TRANSLUCENCY_SHADOWS

	#if CORRECT_SHADOW_PCF_BLUR
		Shadow = ApplyPCFOverBlurCorrection(Shadow);
	#endif
	float FadedShadow = lerp(1.0f, Shadow, ShadowFadeFraction * PerObjectDistanceFadeFraction);

#if FORWARD_SHADING || SHADING_PATH_MOBILE 
	float LightInfluenceMask = GetLightInfluenceMask(TranslateWorldPosition);
	// Constrain shadowing from this light to pixels inside the light's influence, since other non-overlapping lights are packed into the same channel
	FadedShadow = lerp(1, FadedShadow, LightInfluenceMask);
	// Write into all channels, the write mask will constrain to the correct one
	OutColor = EncodeLightAttenuation(FadedShadow);
#else
	float FadedSSSTransmittanceOrOpticalDepth = lerp(1.0f, SSSTransmittanceOrOpticalDepth, ShadowFadeFraction * PerObjectDistanceFadeFraction);

	// the channel assignment is documented in ShadowRendering.cpp (look for Light Attenuation channel assignment)
	OutColor = EncodeLightAttenuation(half4(FadedShadow, FadedSSSTransmittanceOrOpticalDepth, FadedShadow, FadedSSSTransmittanceOrOpticalDepth));
#endif

	#if USE_FADE_PLANE || SUBPIXEL_SHADOW
	// When the fade plane is in use for CSMs, we output the fade value in the alpha channel for blending.
	if (bUseFadePlane)
	{
		OutColor.a = BlendFactor;
	}
	#endif

#if MODULATED_SHADOWS
		OutColor.rgb = lerp(ModulatedShadowColor.rgb, float3(1, 1, 1), FadedShadow);
		OutColor.a = 0;
#endif
}

// .x:DepthBias, y: SlopeDepthBias, z: MaxSlopeDepthBias, 
float3 PointLightDepthBias;

// xy: depth projection parameters
float2 PointLightProjParameters;

/** Pixel shader for projecting a one pass point light shadow from a cube map. */
void MainOnePassPointLightPS(
	in float4 SVPos : SV_POSITION,
	in FStereoPSInput StereoInput,
	out float4 OutColor : SV_Target0
	)
{
	StereoSetupPS(StereoInput);

	float2 ScreenUV = float2( SVPos.xy * ResolvedView.BufferSizeAndInvSize.zw );
#if SHADING_PATH_MOBILE
	float SceneW = CalcSceneDepth(ScreenUV, GetEyeIndex(StereoInput));
#else
	float SceneW = CalcSceneDepth( ScreenUV );
#endif
	
#if SUBPIXEL_SHADOW
	const uint3 PixelCoord = uint3(floor(SVPos.xy), 0);
	const float HairSampleDepth = HairStrands.HairOnlyDepthTexture.Load(PixelCoord).x;
	if (HairSampleDepth > 0)
	{
		SceneW = ConvertFromDeviceZ(HairSampleDepth);
	}
	else
	{
		discard;
	}
#endif

	float2 ScreenPosition = ( ScreenUV.xy - ResolvedView.ScreenPositionScaleBias.wz ) / ResolvedView.ScreenPositionScaleBias.xy;
	float3 TranslateWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneW), SceneW, 1), ResolvedView.ScreenToTranslatedWorld).xyz;

	// For debugging
#define OUTPUT_CUBE_SHADOW_DEPTH_NO_FILTERING 0
	float3 WorldSampleToLightVec = LightPositionAndInvRadius.xyz - TranslateWorldPosition.xyz;
#if OUTPUT_CUBE_SHADOW_DEPTH_NO_FILTERING
	// Note: point light shadow depth is Z / W, not linear
#if USE_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE
	float Shadow = TextureCubeSampleLevel(ShadowDepthCubeTexture2, ShadowDepthTextureSampler, WorldSampleToLightVec, 0);
#else
	float Shadow = TextureCubeSampleLevel(ShadowDepthCubeTexture, ShadowDepthTextureSampler, WorldSampleToLightVec, 0);
#endif
#else
	// Normal receiver bias: Increase depth based on the angle with the light and the shading normal. 
	// Unlike spot/rect/direction light, point lights performs depth bias only at projection time. 
	// This is why the slope bias drives the normal receiver bias here.
	float SlopeBias = 0;
#if !SUBPIXEL_SHADOW
	{
#if SUBSTRATE_GBUFFER_FORMAT==1
		const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SVPos.xy, 0)));
		const float3 WorldNormal = TopLayerData.WorldNormal;
#else // SUBSTRATE_GBUFFER_FORMAT==1
	#if SHADING_PATH_MOBILE
		FGBufferData GBufferData = MobileFetchAndDecodeGBuffer(ScreenUV, SVPos);
	#else
		FGBufferData GBufferData = GetGBufferData(ScreenUV);
	#endif
		const float3 WorldNormal = GBufferData.WorldNormal;
#endif // SUBSTRATE_GBUFFER_FORMAT==1

		const float3 LightDirection = normalize(WorldSampleToLightVec);
		const float NoL = saturate(dot(WorldNormal, LightDirection));
		SlopeBias = (1 - NoL) * PointLightDepthBias.y;
	}
#endif

	float Shadow = CubemapHardwarePCF(TranslateWorldPosition, LightPositionAndInvRadius.xyz, LightPositionAndInvRadius.w, PointLightDepthBias.x, SlopeBias, PointLightDepthBias.z);

	Shadow = saturate( (Shadow - 0.5) * ShadowSharpen + 0.5 );

	Shadow = ApplyPCFOverBlurCorrection(Shadow);
#endif

	float FadedShadow = lerp(1.0f, Shadow, ShadowFadeFraction);

#if FORWARD_SHADING
	float LightInfluenceMask = GetLightInfluenceMask(TranslateWorldPosition);
	FadedShadow = lerp(1, FadedShadow, LightInfluenceMask);
	OutColor = EncodeLightAttenuation(FadedShadow);
#else
	// Light attenuation buffer has been remapped. 
	// Point light shadows now write to the blue channel.
	OutColor.b = EncodeLightAttenuation(FadedShadow);
	OutColor.rga = 1;
	// SSS is not correctly handled but at least it should be shadowed
	OutColor.a = OutColor.b;
#endif

#if USE_TRANSMISSION
	float EncodedOpticalDepth = 1.0f;

#if SUBSTRATE_GBUFFER_FORMAT==1

	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(SVPos.xy, 0)));

	const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, SVPos.xy);
	const uint SSSType = SubstrateSubSurfaceHeaderGetSSSType(SSSHeader);

	const bool IsSubsurfaceProfile = SubstrateSubSurfaceHeaderGetUseDiffusion(SSSHeader);
	const float3 WorldNormal = TopLayerData.WorldNormal;
	FTransmissionProfileParams TransmissionParams = GetSubstrateSubsurfaceTransmissionProfileParams(SSSType, SSSHeader);

#else // SUBSTRATE_GBUFFER_FORMAT==1

	// TODO: Use existing GBuffer data, instead of overwriting?
	FGBufferData GBufferData = GetGBufferData(ScreenUV);

	bool IsSubsurfaceProfile = GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE;
	float3 WorldNormal = GBufferData.WorldNormal;
	FTransmissionProfileParams TransmissionParams = GetTransmissionProfileParams(ExtractSubsurfaceProfileInt(GBufferData));

#endif // SUBSTRATE_GBUFFER_FORMAT==1

	//if bSubsurface get SSS shadow,else opaque shadow.
	if (IsSubsurfaceProfile)
	{
		EncodedOpticalDepth = CalculateEncodedOpticalDepth(WorldNormal, TransmissionParams, WorldSampleToLightVec, PointLightProjParameters, LightPositionAndInvRadius.w, 0);
	}

	// Transmittance for Subsurface shading model and Toon is not implemented today.

	OutColor.a = IsSubsurfaceProfile ? EncodeLightAttenuation(EncodedOpticalDepth) : OutColor.b;
#endif // USE_TRANSMISSION
}