// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ShadowProjectionVertexShader.usf: Vertex shader for projecting a shadow depth buffer onto the scene.
=============================================================================*/

#include "Common.ush"

void ShadowProjectionNoTransformVS(
	in float4 InPosition : ATTRIBUTE0,
	in FStereoVSInput StereoInput,
	out float4 OutPosition : SV_POSITION,
	out FStereoVSOutput StereoOutput
)
{
	StereoSetupVS(StereoInput, StereoOutput);
	OutPosition = float4(InPosition.xyz, 1);
}


float4 StencilingGeometryPosAndScale;
float4x4 InvReceiverInnerMatrix;
float4 PreShadowToPreViewTranslation;

void ShadowVolumeBoundProjectionVS(
	in float4 InPosition : ATTRIBUTE0,
	in FStereoVSInput StereoInput,
	out float4 OutPosition : SV_POSITION,
	out FStereoVSOutput StereoOutput
)
{
	StereoSetupVS(StereoInput, StereoOutput);

	float4 UnprojectedVertex = mul(float4(InPosition.xyz, 1), InvReceiverInnerMatrix);
	float3 OutVertex = UnprojectedVertex.xyz / UnprojectedVertex.w + PreShadowToPreViewTranslation.xyz;

	float3 WorldPosition = OutVertex.xyz * StencilingGeometryPosAndScale.w + StencilingGeometryPosAndScale.xyz;
	OutPosition = mul(float4(WorldPosition, 1), ResolvedView.TranslatedWorldToClip);
}


float4 ClipZValues;

void WholeSceneDirectionalShadowStencilVS(
	in float4 InPosition : ATTRIBUTE0,
	in uint VertexID : SV_VertexID,
	out float4 OutPosition : SV_POSITION)
{
	uint IndexInTriangle = VertexID % 3;
	bool FirstTriangle = VertexID < 3;

	OutPosition.x = IndexInTriangle == 1 ? 3.0f : -1.0f;
	OutPosition.y = IndexInTriangle == 2 ? 3.0f : -1.0f;
	OutPosition.z = FirstTriangle ? ClipZValues.x : ClipZValues.y;
	OutPosition.w = 1.0f;

	// Flip facing of second triangle
	OutPosition.x = FirstTriangle ? OutPosition.x : -OutPosition.x;
}
