// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementColorChannelMaskPixelShader.hlsl: Pixel shader for manipulating weights of color channels
=============================================================================*/

#include "Common.ush"

Texture2D InTexture;
SamplerState InTextureSampler;

float4x4 ColorWeights;
float Gamma;

uint bAlphaOnly;

#define WRITE_TO_GBUFFER (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float4 BaseColor = Texture2DSample(InTexture, InTextureSampler,TextureCoordinate);
	float4 FinalColor;

	// Seperate the Color weights and use against the Base colour to detrmine the actual colour from our filter
	FinalColor.r = dot(BaseColor, ColorWeights[0]);
	FinalColor.g = dot(BaseColor, ColorWeights[1]);
	FinalColor.b = dot(BaseColor, ColorWeights[2]);
	FinalColor.a = dot(BaseColor, ColorWeights[3]);
		 
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		FinalColor.rgb = pow(saturate(FinalColor.rgb),Gamma);
	}
		
	FinalColor*=Color;
	
	OutColor = RETURN_COLOR(FinalColor);	

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}