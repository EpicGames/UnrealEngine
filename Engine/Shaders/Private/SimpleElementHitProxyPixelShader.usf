// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementHitProxyPixelShader.hlsl: Pixel shader for drawing simple element hit proxies.
=============================================================================*/

#include "Common.ush"

Texture2D InTexture;
SamplerState InTextureSampler;

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 HitProxyId : TEXCOORD2,
	out float4 OutColor : SV_Target0
	)
{
	float alpha = Texture2DSample(InTexture, InTextureSampler,TextureCoordinate).a;
	OutColor = float4(HitProxyId.rgb,alpha);

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	// Dx11 (feature levels 10 and upwards) do not perform alpha test, so manually clip.
	clip(alpha - 1.f/255.f);
#endif
}
