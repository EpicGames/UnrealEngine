// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementNormalMapPixelShader.hlsl: Pixel shader for previewing normal
		map textures rendered as simple elements.

Deprecated, do not use.

Use SimpleElementTexture2DPreviewPixelShader.usf instead, with the normal map flag.

=============================================================================*/

#include "Common.ush"

#define WRITE_TO_GBUFFER (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

Texture2D NormalMapTexture;
SamplerState NormalMapTextureSampler;

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	const float4 NormalSample = Texture2DSample(NormalMapTexture, NormalMapTextureSampler, TextureCoordinate);
	const float4 Normal = UnpackNormalMap(NormalSample);
	const float4 RescaledNormal = float4(Normal.xyz * 0.5 + 0.5, 1);
	OutColor = RETURN_COLOR(RescaledNormal);	

/*
	// this shader does no gamma correction
	if( Gamma != 1.0 )
	{
		// typically Gamma = 1.0/2.2
		// then this does LinearToSrgb
		OutColor.rgb = ApplyGammaCorrection(saturate(OutColor.rgb), Gamma * 2.2);
	}
*/

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}
