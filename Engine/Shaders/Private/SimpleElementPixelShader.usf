// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementPixelShader.hlsl: Pixel shader for drawing simple elements.
=============================================================================*/

#include "Common.ush"
#include "ColorUtils.ush"
#include "GammaCorrectionCommon.ush"
#include "IESLightProfilesCommon.ush"

#define ENABLE_EDITOR_COMPOSITING (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 || MOBILE_EMULATION)
#define WRITE_TO_SHADING_MODEL (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

#ifndef TEXTURECUBE_ARRAY
#define TEXTURECUBE_ARRAY 0
#endif

Texture2D InTexture;
SamplerState InTextureSampler;

half4 TextureComponentReplicate;
half4 TextureComponentReplicateAlpha;

void ReplicateChannelSimpleElementShader(inout float4 BaseColor)
{
	ReplicateChannel(BaseColor, TextureComponentReplicate, TextureComponentReplicateAlpha);
}

MaterialFloat4 ColourTexture2DSample(Texture2D Tex, SamplerState Sampler, float2 UV)
{
	MaterialFloat4 Sample = Tex.Sample(Sampler, UV);
	return Sample;
}

float3 SimpleElementFrameBufferBlendOp(float4 Source)
{
	half4 Dest = half4(0, 0, 0, 0);

#if SE_BLEND_MODE == SE_BLEND_OPAQUE || SE_BLEND_MODE == SE_BLEND_MASKED || SE_BLEND_MODE == SE_BLEND_MASKEDDISTANCEFIELD || SE_BLEND_MODE == SE_BLEND_MASKEDDISTANCEFIELDSHADOWED
	return Source.rgb;
// AlphaComposite will set both MATERIALBLENDING_TRANSLUCENT and MATERIALBLENDING_ALPHACOMPOSITE defines
// so ensure  MATERIALBLENDING_ALPHACOMPOSITE gets first in line
#elif MATERIALBLENDING_ALPHACOMPOSITE
	return Source.rgb + (Dest.rgb*(1.0 - Source.a));
// AlphaHoldout will set both MATERIALBLENDING_TRANSLUCENT and MATERIALBLENDING_ALPHAHOLDOUT defines
// so ensure  MATERIALBLENDING_ALPHAHOLDOUT gets first in line
#elif MATERIALBLENDING_ALPHAHOLDOUT
	return (Dest.rgb*(1.0 - Source.a));
#elif SE_BLEND_MODE == SE_BLEND_TRANSLUCENT || SE_BLEND_MODE == SE_BLEND_TRANSLUCENTALPHAONLY || SE_BLEND_MODE == SE_BLEND_ALPHABLEND || SE_BLEND_MODE == SE_BLEND_TRANSLUCENTDISTANCEFIELD || SE_BLEND_MODE == SE_BLEND_TRANSLUCENTDISTANCEFIELDSHADOWED || SE_BLEND_MODE == SE_BLEND_TRANSLUCENTALPHAONLYWRITEALPHA
	return (Source.rgb*Source.a) + (Dest.rgb*(1.0 - Source.a));
#elif SE_BLEND_MODE == SE_BLEND_ADDITIVE
	return Source.rgb + Dest.rgb;
#elif SE_BLEND_MODE == SE_BLEND_MODULATE
	return Source.rgb * Dest.rgb;
#elif SE_BLEND_MODE == SE_BLEND_ALPHACOMPOSITE
	return Source.rgb + (Dest.rgb*(1.0 - Source.a));
#elif SE_BLEND_MODE == SE_BLEND_ALPHAHOLDOUT
	return (Dest.rgb*(1.0 - Source.a));
#else
	return Source.rgb;
#endif
}

float4 SimpleElementEncodeFor32BPPHDR(float4 OutColor)
{
	return RETURN_COLOR(OutColor);
}

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 HitProxyId : TEXCOORD2,
	out float4 OutColor : SV_Target0
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1 
#endif
#if ENABLE_EDITOR_COMPOSITING
	,in float4 SvPosition : SV_Position
#endif
	)
{
	float4 BaseColor = ColourTexture2DSample(InTexture, InTextureSampler,TextureCoordinate);

	ReplicateChannelSimpleElementShader(BaseColor);
	OutColor = RETURN_COLOR(BaseColor * Color);	

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif

	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}

void AlphaOnlyMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 HitProxyId : TEXCOORD2,
	out float4 OutColor : SV_Target0
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1 
#endif
#if ENABLE_EDITOR_COMPOSITING
	,in float4 SvPosition : SV_Position
#endif
	)
{
	float4 TmpColor = Color;
	TmpColor.a *= Texture2DSample_A8(InTexture, InTextureSampler, TextureCoordinate);
	OutColor = RETURN_COLOR(TmpColor);

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif

	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}

half Gamma;

void GammaMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 InHitProxyId : TEXCOORD2,			// needed to have d3ddebug warning suppressed (VS and PS signatures with SV_Position
	out float4 OutColor : SV_Target0
#if ENABLE_EDITOR_COMPOSITING
	,in float4 SvPosition : SV_Position
#endif
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1
#endif
	)
{
	float4 BaseColor = ColourTexture2DSample(InTexture, InTextureSampler,TextureCoordinate);
	ReplicateChannelSimpleElementShader(BaseColor);
	OutColor = BaseColor * Color;
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		OutColor.rgb = ApplyGammaCorrection(saturate(OutColor.rgb), 2.2 * Gamma);
	}
	OutColor = RETURN_COLOR(OutColor);

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif

	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}

void GammaAlphaOnlyMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 InHitProxyId : TEXCOORD2,			// needed to have d3ddebug warning suppressed (VS and PS signatures with SV_Position
	out float4 OutColor : SV_Target0
#if ENABLE_EDITOR_COMPOSITING
	,in float4 SvPosition : SV_Position
#endif
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1
#endif
	)
{
	float4 TmpColor = Color;
	TmpColor.a *= Texture2DSample_A8(InTexture, InTextureSampler, TextureCoordinate);
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		TmpColor.rgb = ApplyGammaCorrection(saturate(TmpColor.rgb), 2.2 * Gamma);
	}
	OutColor = RETURN_COLOR(TmpColor);

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif
	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}

float ClipRef;

void GammaMaskedMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 HitProxyId : TEXCOORD2,
	out float4 OutColor : SV_Target0
#if ENABLE_EDITOR_COMPOSITING
	,in float4 SvPosition : SV_Position
#endif
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1
#endif
	)
{
	float4 BaseColor = ColourTexture2DSample(InTexture, InTextureSampler,TextureCoordinate);
	ReplicateChannelSimpleElementShader(BaseColor);
	clip(BaseColor.a - ClipRef);
	OutColor.rgba = BaseColor * Color;
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		OutColor.rgb = ApplyGammaCorrection(saturate(OutColor.rgb), 2.2 * Gamma);
	}
	OutColor = RETURN_COLOR(OutColor);

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif
	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}


/** the width to smooth the edge the texture */
float SmoothWidth;
/** toggles drop shadow rendering */
uint EnableShadow;
/** 2d vector specifying the direction of shadow */
float2 ShadowDirection;
/** Color of the shadowed pixels */
float4 ShadowColor;
/** the width to smooth the edge the shadow of the texture */
float ShadowSmoothWidth;
/** toggles glow rendering */
uint EnableGlow;
/** base color to use for the glow */
float4 GlowColor;
/** outline glow outer radius */
float2 GlowOuterRadius;
/** outline glow inner radius */
float2 GlowInnerRadius;

/**
* Distance field rendering of textured tile.
* Alpha value represents distance to silhouette edge. A value of 0.5 is exactly on the edge.
*
* Based on the following paper:
* http://www.valvesoftware.com/publications/2007/SIGGRAPH2007_AlphaTestedMagnification.pdf
*/
void GammaDistanceFieldMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	out float4 OutColor : SV_Target0
	)
{
	float4 BaseTexture = Texture2DSample(InTexture, InTextureSampler,TextureCoordinate);
	ReplicateChannelSimpleElementShader(BaseTexture);
	float BaseDist = BaseTexture.a;
	// smooth range between SmoothWidth texels around edge
	// We want to control font edge smoothness in screen space, but the distance field is stored in texture space.
	// Using ddx scales the smoothstep steepness based on texel-to-pixel scaling.
	float TextureToScreenCompensation = ddx(TextureCoordinate.x);
	float SmoothstepWidth = SmoothWidth * TextureToScreenCompensation;
	float4 BaseColor = float4(Color.rgb, smoothstep(0.5-SmoothstepWidth, 0.5+SmoothstepWidth, BaseDist));
	
// @todo-mobile: bool doesn't work on ES2 (but it should in ES31? Why is 31 in this check)
#if !(ES3_1_PROFILE)
	if( EnableShadow )
	{
		// sample texture with shadow direction offset
		float4 ShadowTexture = Texture2DSample(InTexture, InTextureSampler,TextureCoordinate + ShadowDirection);
		ReplicateChannelSimpleElementShader(ShadowTexture);
		float ShadowDist = ShadowTexture.a;

		// smooth range between ShadowSmoothWidth texels around edge
		float4 ShadowResult = ShadowColor * smoothstep(0.5-ShadowSmoothWidth, 0.5+ShadowSmoothWidth, ShadowDist);
		BaseColor =  lerp(ShadowResult, BaseColor, BaseColor.a);
	}

	if (EnableGlow)
	{
		// testing glow rendering
		//GlowColor = float4(0, 1, 0, 1);
		//GlowOuterRadius = float2(0.45,0.5);
		//GlowInnerRadius = float2(0.5,0.51);	

		if (BaseDist >= GlowOuterRadius[0] && BaseDist <= GlowInnerRadius[1])
		{
			float OutlineFactor=0.0;
			if (BaseDist <= GlowOuterRadius[1])
			{
				OutlineFactor = smoothstep(GlowOuterRadius[0],GlowOuterRadius[1],BaseDist);
				BaseColor = lerp(float4(GlowColor.rgb, 0.0), GlowColor, OutlineFactor);
			}
			else
			{
				OutlineFactor = smoothstep(GlowInnerRadius[1],GlowInnerRadius[0],BaseDist);
				BaseColor = lerp(BaseColor, GlowColor, OutlineFactor);
			}
			
		}
	}
 #endif
	
	clip(BaseColor.a - ClipRef);
	OutColor = BaseColor;
	OutColor.a *= Color.a;
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		OutColor.rgb = ApplyGammaCorrection(saturate(OutColor.rgb), 2.2 * Gamma);
	}

	OutColor = SimpleElementEncodeFor32BPPHDR(OutColor);
}

#if TEXTURECUBE_ARRAY
TextureCubeArray CubeTexture;
#else
TextureCube CubeTexture;
#endif

SamplerState CubeTextureSampler;

// .x:MipLevel, .y:bShowLongLatUnwrap, .zw:unused
float4 PackedProperties0;
float4x4 ColorWeights;
float4x4 ViewMatrix;

#if TEXTURECUBE_ARRAY
float NumSlices;
float SliceIndex;
#endif

// used in the Texture info dialog when looking at a cubemap texture
void CubemapTextureProperties(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 InterpolatedColor : TEXCOORD1,
	out float4 OutColor : SV_Target0
#if WRITE_TO_SHADING_MODEL
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float MipLevel = PackedProperties0.x;
	float bShowLongLatUnwrap = PackedProperties0.y;
	float3 CubeCoordinates;
	float4 BaseColor;

	if (bShowLongLatUnwrap > 0.0)
	{
		float2 Angles = float2(2 * PI * (TextureCoordinate.x + 0.5f), PI * TextureCoordinate.y);
		float s = sin(Angles.y);
		CubeCoordinates = float3(s * sin(Angles.x), -s * cos(Angles.x), cos(Angles.y));
	}
	else
	{
		float2 ScaledUVs = TextureCoordinate * 2 - 1;
		// identity view matrix displays the face 0 properly oriented in space
		CubeCoordinates = mul(float3(1, ScaledUVs.x, -ScaledUVs.y), (float3x3)ViewMatrix);
	}

#if TEXTURECUBE_ARRAY
	// SliceIndex represents an index of a cubemap in the array
	float CurrentSliceIndex = SliceIndex;
	if (CurrentSliceIndex < 0.0)
	{
		// if the slice index is not specified, display a portion of each cubemap from the TextureCube array
		CurrentSliceIndex = floor(TextureCoordinate.y * NumSlices);
	}
#endif

	if (MipLevel >= 0.0)
	{
#if TEXTURECUBE_ARRAY
		BaseColor = TextureCubeArraySampleLevel(CubeTexture, CubeTextureSampler, float4(CubeCoordinates, CurrentSliceIndex), MipLevel);
#else
		BaseColor = TextureCubeSampleLevel(CubeTexture, CubeTextureSampler, CubeCoordinates, MipLevel);
#endif
	}
	else
	{
#if TEXTURECUBE_ARRAY
		BaseColor = TextureCubeArraySample(CubeTexture, CubeTextureSampler, float4(CubeCoordinates, CurrentSliceIndex));
#else
		BaseColor = TextureCubeSample(CubeTexture, CubeTextureSampler, CubeCoordinates);
#endif
	}

	float4 FinalColor;

	// Seperate the Color weights and use against the Base colour to detrmine the actual colour from our filter
	FinalColor.r = dot(BaseColor, ColorWeights[0]);
	FinalColor.g = dot(BaseColor, ColorWeights[1]);
	FinalColor.b = dot(BaseColor, ColorWeights[2]);
	FinalColor.a = dot(BaseColor, ColorWeights[3]);

	FinalColor *= InterpolatedColor;

#if HDR_OUTPUT == 1
	FinalColor.rgb =max(float3(0,0,0),FinalColor.rgb);	
#else
	if( Gamma != 1.0 )
	{
		// Gamma correct the output color.
		FinalColor.rgb = ApplyGammaCorrection(saturate(FinalColor.rgb), 2.2 * Gamma);
	}
#endif
	
	OutColor = RETURN_COLOR(FinalColor);	

#if WRITE_TO_SHADING_MODEL
	// Set the G buffer bits that indicate unlit
	OutWorldNormal = 0;
#endif
}

float3 ExtractLargestComponent(float3 v)
{
	float3 a = abs(v);

	if(a.x > a.y)
	{
		if(a.x > a.z)
		{
			return float3(v.x > 0 ? 1 : -1, 0, 0);
		}
	}
	else
	{
		if(a.y > a.z)
		{
			return float3(0, v.y > 0 ? 1 : -1, 0);
		}
	}

	return float3(0, 0, v.z > 0 ? 1 : -1);
}

// from the IES profile
float BrightnessInLumens;

void IESLightProfileMain(
	in float2 TextureCoordinate : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	float3 BoxExtent = float3(2, 1.0f, 1);

	float TiltAngle = 0.3f;
	// y is downwards
	// z is pointing into the screen
	float3 EyePos = float3(0, 0.8f, 0.3f);
	float3 EyeDirection = float3(TextureCoordinate.xy * 2 - 1, 1) * 20;

	{
		float s = sin(TiltAngle);
		float c = cos(TiltAngle);
		EyeDirection = float3(EyeDirection.x, EyeDirection.y * c - EyeDirection.z * s, EyeDirection.y  * s + EyeDirection.z * c);
	}

	float2 Intersection = LineBoxIntersect(EyePos, EyePos + EyeDirection, -BoxExtent, BoxExtent);
	float3 Position = EyePos + EyeDirection * Intersection.y;
	float3 Normal = ExtractLargestComponent(Position / BoxExtent);

	float3 LightPos = float3(0, -0.25f, 0.85f);

	float LightProfileMultiplier = ComputeLightProfileMultiplier(Position, LightPos, float3(0, 1, 0), float3(0, 0, 1), -1);

	// Inverse square falloff
	float Attenuation = 0.01f / max( 0.01f, length(Position - LightPos));

	// don't preview brightness in thumbnail, arbitrary scale factor for the test scene
	float3 LinearColor = 100.0f * LightProfileMultiplier * Attenuation * saturate(dot(Normal, normalize(Position - LightPos)));

	// preview with actual brightness (some are too dark, others are too bright), arbitrary scale factor to preview light brightness without eye adaptation
//	LinearColor *= 0.0002f * IESMultiplier;

	// simple tonemapper without toe adjustment
	float3 GammaColor;
	{
		half TonemapperRange = 20;
		half TonemapperScaleClamped = 1;
		half A = 0.22 / TonemapperScaleClamped;
		half B = (TonemapperRange + A) / TonemapperRange;
		GammaColor = LinearColor / abs(LinearColor + A) * B;
	}

	OutColor = RETURN_COLOR(float4(GammaColor, 0));
}
