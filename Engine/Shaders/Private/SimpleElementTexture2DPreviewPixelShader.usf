// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementTexture2DPreviewPixelShader.hlsl: Pixel shader for previewing 2d textures and normal maps.
=============================================================================*/

#include "Common.ush"
#include "ColorUtils.ush"
#include "GammaCorrectionCommon.ush"

#ifndef SAMPLE_VIRTUAL_TEXTURE
#define SAMPLE_VIRTUAL_TEXTURE 0
#endif

#ifndef TEXTURE_ARRAY
#define TEXTURE_ARRAY 0
#endif

#if SAMPLE_VIRTUAL_TEXTURE
	#define NUM_VIRTUALTEXTURE_SAMPLES 1
	// Workaround for unbound View uniform buffer
	#define VT_DISABLE_VIEW_UNIFORM_BUFFER 1
	#include "VirtualTextureCommon.ush"
#endif

#define WRITE_TO_GBUFFER (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

SamplerState InTextureSampler;

#if SAMPLE_VIRTUAL_TEXTURE
	Texture2D<uint4> InPageTableTexture0;
	Texture2D<uint4> InPageTableTexture1;
	Texture2D InPhysicalTexture;

	uint4 VTPackedPageTableUniform[2];
	uint4 VTPackedUniform;
#elif TEXTURE_ARRAY
	Texture2DArray InTextureArray;
#else
	Texture2D InTexture;
#endif

half4 TextureComponentReplicate;
half4 TextureComponentReplicateAlpha;

float4x4 ColorWeights;

//x=Gamma, y=MipLevel, z=bIsNormalMap, w=VT Layer
float4 PackedParams;

#if TEXTURE_ARRAY
float NumSlices;
float SliceIndex;
#endif

void Main(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	in float4 HitProxyId : TEXCOORD2,
	in float4 InPosition : SV_POSITION,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	float Gamma = PackedParams.x;
	float MipLevel = PackedParams.y;
	float bIsNormalMap = PackedParams.z;
	uint LayerIndex = (uint)PackedParams.w;

	float4 FinalColor;
	float4 Sample;

#if TEXTURE_ARRAY
	float CurrentSliceIndex = SliceIndex;
	if (CurrentSliceIndex < 0.0)
	{
		// If the slice index is not specified, display a portion of each slice from the texture array.
		CurrentSliceIndex = floor(TextureCoordinate.y * NumSlices);
	}
	float3 SampleCoordinates = float3(TextureCoordinate.xy, CurrentSliceIndex);

	if( MipLevel >= 0.0f )
	{
		Sample = Texture2DArraySampleLevel(InTextureArray, InTextureSampler, SampleCoordinates, MipLevel);
	}
	else
	{
		Sample = Texture2DArraySample(InTextureArray, InTextureSampler, SampleCoordinates);
	}
#else //TEXTURE_ARAY
	if( MipLevel >= 0.0f )
	{
#if SAMPLE_VIRTUAL_TEXTURE
		VTPageTableResult PageTableResult = TextureLoadVirtualPageTableLevel(InPageTableTexture0, InPageTableTexture1, VTPageTableUniform_Unpack(VTPackedPageTableUniform[0], VTPackedPageTableUniform[1]), TextureCoordinate, VTADDRESSMODE_WRAP, VTADDRESSMODE_WRAP, MipLevel);
		Sample = TextureVirtualSampleLevel( InPhysicalTexture, InTextureSampler, PageTableResult, LayerIndex, VTUniform_Unpack(VTPackedUniform));
#else
		Sample = Texture2DSampleLevel(InTexture, InTextureSampler,TextureCoordinate,MipLevel);
#endif
	}
	else
	{
#if SAMPLE_VIRTUAL_TEXTURE
		VTPageTableResult PageTableResult = TextureLoadVirtualPageTable(InPageTableTexture0, InPageTableTexture1, VTPageTableUniform_Unpack(VTPackedPageTableUniform[0], VTPackedPageTableUniform[1]), TextureCoordinate, VTADDRESSMODE_WRAP, VTADDRESSMODE_WRAP, 0.0f, InPosition.xy);
		Sample = TextureVirtualSample( InPhysicalTexture, InTextureSampler, PageTableResult, LayerIndex, VTUniform_Unpack(VTPackedUniform));
#else
		Sample = Texture2DSample(InTexture, InTextureSampler,TextureCoordinate);
#endif
	}
#endif //TEXTURE_ARRAY

	ReplicateChannel(Sample,TextureComponentReplicate,TextureComponentReplicateAlpha);

	if( bIsNormalMap >= 0.0 )
	{
		const float4 Normal = UnpackNormalMap(Sample);
		const float4 RescaledNormal = float4(Normal.xyz * 0.5 + 0.5, 1);
		// RETURN_COLOR is typically a nop
		Sample = RETURN_COLOR(RescaledNormal);
	}

	// Seperate the Color weights and use against the Base colour to detrmine the actual colour from our filter
	FinalColor.r = dot(Sample, ColorWeights[0]);
	FinalColor.g = dot(Sample, ColorWeights[1]);
	FinalColor.b = dot(Sample, ColorWeights[2]);
	FinalColor.a = dot(Sample, ColorWeights[3]);

	FinalColor *= Color;

	if( Gamma != 1.0 )
	{
		// typically Gamma = 1.0/2.2
		// then this does LinearToSrgb
		FinalColor.rgb = ApplyGammaCorrection(saturate(FinalColor.rgb), Gamma * 2.2);
	}
	
	FinalColor = RETURN_COLOR(FinalColor);	
		 
	OutColor = FinalColor;

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}