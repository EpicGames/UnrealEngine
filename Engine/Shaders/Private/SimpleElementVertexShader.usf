// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementVertexShader.hlsl: Vertex shader for drawing simple elements.
=============================================================================*/

#include "Common.ush"

#ifndef ENABLE_LWC
#define ENABLE_LWC 0
#endif

float4x4 Transform[2];
#if ENABLE_LWC
float3 TransformPositionHigh[2];
#endif

void Main(
	in float4 InPositionHigh : ATTRIBUTE0,
	in float4 InPositionLow : ATTRIBUTE1,
	in float2 InTextureCoordinate : ATTRIBUTE2,
	in float4 InColor : ATTRIBUTE3,
	in float4 InHitProxyId : ATTRIBUTE4,
#if INSTANCED_STEREO
    in uint InstanceId : SV_InstanceID,
#elif MOBILE_MULTI_VIEW
	in nointerpolation uint ViewId : SV_ViewID,
#endif
	out float2 OutTextureCoordinate : TEXCOORD0,
	out float4 OutColor : TEXCOORD1,
	out float4 OutHitProxyId : TEXCOORD2,
	out float4 OutPosition : SV_POSITION
#if INSTANCED_STEREO
	, out uint ViewportIndex : SV_ViewPortArrayIndex
#endif
	)
{
#if INSTANCED_STEREO
    const uint EyeIndex = InstanceId;
	ViewportIndex = EyeIndex;
#elif MOBILE_MULTI_VIEW
	const uint EyeIndex = ViewId;
#else
	const uint EyeIndex = 0;	
#endif

#if ENABLE_LWC
	FDFVector4 InPosition = MakeDFVector4(InPositionHigh, InPositionLow);
	FDFInverseMatrix WorldToClip = MakeDFInverseMatrix(TransformPositionHigh[EyeIndex], Transform[EyeIndex]);
	OutPosition = DFMultiplyDemote(InPosition, WorldToClip);
	OutPosition.z = max(OutPosition.z, 0);
#else
	OutPosition = mul(InPositionHigh, Transform[EyeIndex]);
#endif

	OutTextureCoordinate = InTextureCoordinate;
	OutColor = InColor;
	OutHitProxyId = InHitProxyId;
}
