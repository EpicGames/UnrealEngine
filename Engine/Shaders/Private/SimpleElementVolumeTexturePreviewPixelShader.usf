// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimpleElementVolumeTexturePreviewPixelShader.hlsl: Pixel shader for previewing volume textures.
=============================================================================*/

#include "Common.ush"
#include "ColorUtils.ush"
#include "GammaCorrectionCommon.ush"

#define WRITE_TO_GBUFFER (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

Texture3D InTexture;
SamplerState InTextureSampler;

Texture2D BadTexture;
SamplerState BadTextureSampler;

float4x4 ColorWeights;

//x=Gamma, y=MipLevel, z=SizeZ, w=Density
float4 PackedParams;
float4 NumTilesPerSide;

// This defines the proportion of each side of the volume. (w is the max)
float4 TraceVolumeScaling;

// Texture dimension : width, heigth and depth.
float3 TextureDimension;

// Transform into view space.
float4x4 TraceViewMatrix;

void TileMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	const float Gamma = PackedParams.x;
	const float MipLevel = PackedParams.y;
	float TexSizeZ = PackedParams.z;

	TextureCoordinate *= NumTilesPerSide.xy;

	float TileIndexZ = floor(TextureCoordinate.x) + floor(TextureCoordinate.y) * NumTilesPerSide.x;
	float3 TexCoord3D = float3(frac(TextureCoordinate), (TileIndexZ + .5) / TexSizeZ);

	bool bLayerIsInvalid = TexCoord3D.z > 1;

	float2 PixelOffset = float2(ddx(TextureCoordinate.x), ddy(TextureCoordinate.y));
	bool bIsBorder = any(floor(TextureCoordinate) != floor(TextureCoordinate + PixelOffset));

	float4 FinalColor;
	float4 Sample;

	if( MipLevel >= 0.0f )
	{
		Sample = Texture3DSampleLevel(InTexture, InTextureSampler, TexCoord3D, MipLevel);
	}
	else
	{
		Sample = Texture3DSampleGrad(InTexture, InTextureSampler, TexCoord3D, ddx(float3(TextureCoordinate, 0)), ddy(float3(TextureCoordinate, 0)));
	}


	float4 BadSample = Texture2DSample(BadTexture, BadTextureSampler, TexCoord3D.xy);

	// Seperate the Color weights and use against the Base colour to detrmine the actual colour from our filter
	FinalColor.r = dot(Sample, ColorWeights[0]);
	FinalColor.g = dot(Sample, ColorWeights[1]);
	FinalColor.b = dot(Sample, ColorWeights[2]);
	FinalColor.a = dot(Sample, ColorWeights[3]);
	FinalColor *= Color;

	// Invert color at the border.
	FinalColor.rgb = bIsBorder ? frac(FinalColor.rgb + .5) : FinalColor.rgb;

	// Show bad sample for invalid tiles.
	FinalColor = bLayerIsInvalid ? BadSample : FinalColor;

	if( Gamma != 1.0 )
	{
		FinalColor.rgb = ApplyGammaCorrection(saturate(FinalColor.rgb), 2.2 / (1.0 / Gamma));
	}
	
	FinalColor = RETURN_COLOR(FinalColor);
	OutColor = FinalColor;

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Rotates Position about the given axis by the given angle, in radians, and returns the offset to Position. */
float3 RotateAboutAxis(float4 NormalizedRotationAxisAndAngle, float3 PositionOnAxis, float3 Position)
{
	// Project Position onto the rotation axis and find the closest point on the axis to Position
	float3 ClosestPointOnAxis = PositionOnAxis + NormalizedRotationAxisAndAngle.xyz * dot(NormalizedRotationAxisAndAngle.xyz, Position - PositionOnAxis);
	// Construct orthogonal axes in the plane of the rotation
	float3 UAxis = Position - ClosestPointOnAxis;
	float3 VAxis = cross(NormalizedRotationAxisAndAngle.xyz, UAxis);
	float CosAngle;
	float SinAngle;
	sincos(NormalizedRotationAxisAndAngle.w, SinAngle, CosAngle);
	// Rotate using the orthogonal axes
	float3 R = UAxis * CosAngle + VAxis * SinAngle;
	// Reconstruct the rotated world space position
	float3 RotatedPosition = ClosestPointOnAxis + R;
	// Convert from position to a position offset
	return RotatedPosition - Position;
}

float2 RayBoxIntersection(float3 RO, float3 RD, float3 Extents )
{
	float3 invraydir = 1 / RD;

	float3 firstintersections = (-Extents - RO) * invraydir;
	float3 secondintersections = (Extents - RO) * invraydir;
	float3 closest = min(firstintersections, secondintersections);
	float3 furthest = max(firstintersections, secondintersections);

	float t0 = max(closest.x, max(closest.y, closest.z));
	float t1 = min(furthest.x, min(furthest.y, furthest.z));

	t0 = max(0, t0);
	t1 = max(0, t1);
	t0 = min(t0, t1);

	return float2(t0, t1);
}

float3x3 GetTraceViewMatrix()
{
	float3 CameraX = normalize(float3(-1, 0.1,0));
	float rot = frac( View.GameTime * 0.1) * 2 * PI;
	CameraX = RotateAboutAxis( float4( float3(0,0,1) , rot) , 0 , CameraX) + CameraX;
	float3 CameraY = normalize( cross( CameraX, float3(0,0,1) ) );
	float3x3 ViewMatrix = { CameraX, CameraY, -normalize(cross( CameraX, CameraY ) ) };
	return ViewMatrix;
}

void GetTexCoordAndIncrement(in float2 TextureCoordinate, out int NumSteps, out float boxthickness, out float edgeonly, out float3 TexCoord, out float3 Increment)
{
	const float MaxSteps = 128;

	//View Local->World transform
	float3x3 ViewMatrix = (float3x3)TraceViewMatrix; // GetTraceViewMatrix();

	//Allow FOV to be specified in Degrees via settings
	const float FOV = 90;
	const float ZVec = 1 / tan(FOV / 2 / 57.295799);
	float CameraDistance = (TraceVolumeScaling.w / ( 0.5 / ZVec )) + TraceVolumeScaling.w;

	//Setup Clip Space
	float2 UV = (TextureCoordinate - 0.5) * float2(1,-1);
	UV *= 1.5;

	//X forward camera basis
	float3 RO =  -float3( CameraDistance, 0, 0 );
	float3 RD =  normalize( float3( ZVec, UV ) );

	//Bring vectors into view space
	float3 localcampos = mul(RO, ViewMatrix);
	float3 localcamvec = normalize(mul(RD, ViewMatrix));

	//Box Intersection
	float2 box1 = RayBoxIntersection( localcampos, localcamvec, TraceVolumeScaling.xyz * 0.5);
	float t0 = box1.x;
	float t1 = box1.y;
	
	float planeoffset = 1-frac( ( t0 - length(localcampos) ) * MaxSteps );

	t0 += (planeoffset / MaxSteps);
	t1 += (planeoffset / MaxSteps);

	float3 entrypos = localcampos + t0 * localcamvec;
	float3 exitpos  = localcampos + t1 * localcamvec;

	boxthickness = length( (entrypos - exitpos) / TraceVolumeScaling.xyz);

	//second smaller box
	float2 box2 = RayBoxIntersection( localcampos, localcamvec, TraceVolumeScaling.xyz * 0.4975);
	t0 = box2.x;
	t1 = box2.y;

	entrypos = localcampos + t0 * localcamvec;
	exitpos  = localcampos + t1 * localcamvec;

	edgeonly = length( (entrypos - exitpos) / TraceVolumeScaling.xyz);
	edgeonly = boxthickness - edgeonly;

	float StepSize = 1.f / MaxSteps;

	// Use "round" and not "ceil" otherwise NumSteps > 0 even when outside the box.
	NumSteps = round(boxthickness * MaxSteps);

	float3 CurPos = ( entrypos / TraceVolumeScaling.xyz ) + 0.5;
	localcamvec /= TraceVolumeScaling.xyz;

	// Result

	TexCoord = CurPos;
	Increment = localcamvec * StepSize;
}

void TraceMain(
	in float2 TextureCoordinate : TEXCOORD0,
	in float4 Color : TEXCOORD1,
	out float4 OutColor : SV_Target0
#if WRITE_TO_GBUFFER
	,out float4 OutWorldNormal : SV_Target1 
#endif
	)
{
	const float Gamma = PackedParams.x;
	const float MipLevel = PackedParams.y;
	const float Density = PackedParams.w;

	float AmbientDensity = 1;

	float4 Accum4 = 0;
	float Transmittance = 1;

	float boxthickness = 0;
	float edgeonly = 0;

	float3 TexCoord3D = 0;
	float3 Increment3D = 0;
	int NumSteps = 0;
	GetTexCoordAndIncrement(TextureCoordinate, NumSteps, boxthickness, edgeonly, TexCoord3D, Increment3D);

	// Discard when NumSteps = 0
	clip(NumSteps - .5f);

	const float NumTexelsPerSample = dot(abs(Increment3D), TextureDimension);

	LOOP
	for (int i = 0; i < NumSteps && Transmittance > .02; i++)
	{
		float4 CurSample = saturate(Texture3DSampleLevel(InTexture, InTextureSampler, saturate(float3(1 - TexCoord3D.x, TexCoord3D.y, TexCoord3D.z)), MipLevel));

		// Compute the current step 
		float CurDensity = dot(CurSample, ColorWeights[3]) * Density;
		CurDensity = saturate(1 - pow(1 - CurDensity, NumTexelsPerSample));

		Accum4 += Transmittance * CurDensity * CurSample;
		Transmittance *= 1 - CurDensity;

		TexCoord3D += Increment3D;
	}

	float3 Accum = 0;
	Accum.r = dot(Accum4, ColorWeights[0]);
	Accum.g = dot(Accum4, ColorWeights[1]);
	Accum.b = dot(Accum4, ColorWeights[2]);

	float4 FinalColor = float4(Accum, 1 - Transmittance);

	if( Gamma != 1.0 )
	{
		FinalColor.rgb = ApplyGammaCorrection(saturate(FinalColor.rgb), 2.2 / (1.0 / Gamma));
	}
	
	FinalColor = RETURN_COLOR(FinalColor);
	OutColor = FinalColor;

#if WRITE_TO_GBUFFER
	// Set the G buffer bits that indicate that deferred lighting and image reflections are not enabled
	OutWorldNormal = 0;
#endif
}