// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SimplePixelShader_F32.usf:  pixel shader source.
=============================================================================*/

sampler2D SourceTexture;

void main( float2 UV : TEXCOORD0, out float4 OutColor : COLOR0 )
{
	float Result = tex2D( SourceTexture, UV ).r;
	OutColor = float4( Result, Result, Result, Result );
}
