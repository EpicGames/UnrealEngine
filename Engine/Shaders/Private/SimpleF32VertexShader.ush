// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DownsampleVertexShader_F32.usf: Filter vertex shader source.
=============================================================================*/

void main(
	float4 Position : POSITION,
	float2 UV : TEXCOORD0,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : POSITION
	)
{
	OutPosition = Position;
	OutUV = UV;
}
