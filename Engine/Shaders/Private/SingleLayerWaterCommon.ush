// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TextureSampling.ush" // GetBilinearSampleLevelInfos() and GetBilinearInterpolation()

// Full shading quality, sampling both the color and depth behind the water surface and supporting refraction.
// Used in BasePassPixelShader.usf.
#define SINGLE_LAYER_WATER_SHADING_QUALITY_FULL							0

// Reduced shading quality (same as mobile), sampling only the depth behind the water surface and does not support refraction.
// Blended against the background using HW blending.
// This quality level also causes BasePassPixelShader.usf to be compiled without forward directional light shadow and is likely to be used
// together with DISABLE_FORWARD_LOCAL_LIGHTS to disable all local lights.
// Used in BasePassPixelShader.usf.
#define SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE	1

// Reduced shading quality for mobile platforms, sampling only the depth buffer behind the water using LookupDeviceZ(). Does not support refraction.
// Blended against the background using HW blending.
// Used in MobileBasePassPixelShader.usf.
#define SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_BUFFER		2

#ifndef SINGLE_LAYER_WATER_USE_REFRACTION_MASK
#define SINGLE_LAYER_WATER_USE_REFRACTION_MASK 1
#endif

#define SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE ((SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_FULL) && SINGLE_LAYER_WATER_USE_REFRACTION_MASK) 

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	#if SUPPORTS_INDEPENDENT_SAMPLERS
		#define SingleLayerWaterSceneColorSampler View.SharedBilinearClampedSampler
		#if METAL_ES3_1_PROFILE
			#define SingleLayerWaterSceneDepthSampler View.SharedPointClampedSampler
		#else // TODO: Dynamically use a point sampler instead if the depth texture was not downsampled.
			#define SingleLayerWaterSceneDepthSampler View.SharedBilinearClampedSampler
		#endif // METAL_ES3_1_PROFILE
	#else
		#define SingleLayerWaterSceneColorSampler SingleLayerWater.SceneColorWithoutSingleLayerWaterSampler
		#define SingleLayerWaterSceneDepthSampler SingleLayerWater.SceneDepthWithoutSingleLayerWaterSampler
	#endif // SUPPORTS_INDEPENDENT_SAMPLERS
#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER

// Returns the safe UV bounds for sampling the SceneColor/SceneDepthWithoutWater refraction textures.
// Derived from ResolvedView's per-eye rect, so each eye stays inside its own region under ISR.
// floor() matches FIntPoint::DivideAndRoundDown used to size the refraction copy viewport, keeping
// bounds inside the actually-written region when ViewRect doesn't divide evenly by the refraction
// downsample factor (e.g. dynamic resolution).
float4 GetSingleLayerWaterRefractionUVMinMax(float2 RefractionTextureSize, float2 RefractionTextureInvSize)
{
	const float2 EyeViewMin = ResolvedView.ViewRectMin.xy;
	const float2 EyeViewMax = EyeViewMin + ResolvedView.ViewSizeAndInvSize.xy;
	const float2 BufferToRefraction = View.BufferSizeAndInvSize.zw * RefractionTextureSize;
	const float PixelSafeGuardBand = 0.55f; // Gather4-safe guard band (matches PixelSafeGuardBand in SingleLayerWaterRendering.cpp).
	const float2 MinRefraction = floor(EyeViewMin * BufferToRefraction) + PixelSafeGuardBand;
	const float2 MaxRefraction = floor(EyeViewMax * BufferToRefraction) - PixelSafeGuardBand;
	return float4(MinRefraction, MaxRefraction) * RefractionTextureInvSize.xyxy;
}

// We use our own refract because the HLSL one was not working on all platforms (ES3.1 preview for instance)
bool WaterRefract(float3 V, float3 N, float Eta, out float3 R)
{
	const float CosThetaI = dot(V, N);
	const float CosThetaT2 = 1.f - Eta * Eta * (1.0f - CosThetaI * CosThetaI);
	if (CosThetaT2 < 0.0f)
	{
		// Total internal reflection
		R = reflect(-V, N);
		return false;
	}

	const float CosThetaT = -(CosThetaI >= 0.0f ? 1.0f : -1.0f) * sqrt(CosThetaT2);
	R = (Eta * CosThetaI + CosThetaT) * N - Eta * V;
	return true;
}

float WaterSampleSceneDepthWithoutWater(Texture2D SceneDepthWithoutWaterTexture, SamplerState SceneDepthWithoutWaterSampler, float2 UV, float2 SceneDepthWithoutWaterTextureSize, float2 SceneDepthWithoutWaterTextureInvSize)
{
#if METAL_ES3_1_PROFILE
	// On Metal we cannot do hardware bilinear sampling of float32 textures, so we need to do it manually.
	FBilinearSampleInfos SampleInfos = GetBilinearSampleLevelInfos(UV, SceneDepthWithoutWaterTextureSize, SceneDepthWithoutWaterTextureInvSize);
	float4 PixelSceneDepth4 = SceneDepthWithoutWaterTexture.Gather(SceneDepthWithoutWaterSampler, UV).wzxy;
	return GetBilinearInterpolation(SampleInfos, PixelSceneDepth4);
#else
	// A linear sampler should be passed in here if downsampling was used.
	return SceneDepthWithoutWaterTexture.SampleLevel(SceneDepthWithoutWaterSampler, UV, 0).x;
#endif // METAL_ES3_1_PROFILE
}

#define SLW_MASK_IS_WATER_BITMASK (128.0f / 255.0f) // Unorm8 mask with 8th bit set
#define SLW_MASK_REFRACTION_BITMASK (127.0f / 255.0f) // Unorm8 mask with the first 7 bits set

float PackWaterRefractionMask(bool bIsWater, float RefractionMask)
{
	if (bIsWater)
	{
		return saturate(RefractionMask) * SLW_MASK_REFRACTION_BITMASK + SLW_MASK_IS_WATER_BITMASK; // Encode refraction mask in first 7 bits and water mask in 8th bit, assuming a R8 render target
	}
	return 0.0f;
}

void UnpackWaterRefractionMask(float PackedRefractionMask, out bool bOutIsWater, out float OutRefractionMask)
{
	bOutIsWater = PackedRefractionMask >= SLW_MASK_IS_WATER_BITMASK;
	OutRefractionMask = bOutIsWater ? (PackedRefractionMask - SLW_MASK_IS_WATER_BITMASK) / SLW_MASK_REFRACTION_BITMASK : 0.0f;
}

void UnpackWaterRefractionMask(float4 PackedRefractionMask, out bool4 bOutIsWater, out float4 OutRefractionMask)
{
	bOutIsWater = PackedRefractionMask >= SLW_MASK_IS_WATER_BITMASK;
	OutRefractionMask = select(bOutIsWater, (PackedRefractionMask - SLW_MASK_IS_WATER_BITMASK) / SLW_MASK_REFRACTION_BITMASK, 0.0f);
}

#undef SLW_MASK_IS_WATER_BITMASK
#undef SLW_MASK_REFRACTION_BITMASK

void SampleSceneDepthWithoutWaterBilateral(Texture2D Tex, SamplerState TexSampler, float2 UV, float2 TexSize, float2 InvTexSize, float WaterSurfaceSceneDepth,
#if SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE
	Texture2D RefractionMaskTex,
#endif
	out float OutDeviceZ, out float OutSceneDepth, out float4 OutWeights, out float4 OutRefractionMask)
{
	// Bias UV by ~1/512 of a texel to avoid sitting exactly on Gather block boundaries,
	// where GPU fixed-point and shader float32 arithmetic can disagree on block selection.
	UV += (0.5f / 256.0f) * InvTexSize;
	// Gather deviceZ depth values and convert to scene depth
	const float4 DeviceZ4 = Tex.Gather(TexSampler, UV);
	const float4 SceneDepth4 = float4(ConvertFromDeviceZ(DeviceZ4.x), ConvertFromDeviceZ(DeviceZ4.y), ConvertFromDeviceZ(DeviceZ4.z), ConvertFromDeviceZ(DeviceZ4.w));

	// Standard bilinear interpolation factors
	const float2 BilinearInterp = frac(UV * TexSize - 0.5f);
	const float4 BilinearWeights = float4(
		(1.0f - BilinearInterp.x) * BilinearInterp.y,
		BilinearInterp.x * BilinearInterp.y,
		BilinearInterp.x * (1.0f - BilinearInterp.y),
		(1.0f - BilinearInterp.x) * (1.0f - BilinearInterp.y));

	// Heuristic to reject non-water pixels belonging to foreground objects based on depth
	float SlopeTolerance = max(abs(ddx_fine(WaterSurfaceSceneDepth)), abs(ddy_fine(WaterSurfaceSceneDepth)));
	float4 BilateralWeights = select((SceneDepth4 + SlopeTolerance) >= WaterSurfaceSceneDepth, 1.0f, 0.0f);

	// If the refraction mask is available, use it as well
#if SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE
	const float4 PackedRefractionMask4 = RefractionMaskTex.Gather(TexSampler, UV);
	bool4 bIsWater4 = false;
	float4 RefractionMask4 = 0.0f;
	UnpackWaterRefractionMask(PackedRefractionMask4, bIsWater4, RefractionMask4);
	BilateralWeights *= select(bIsWater4, 1.0f, 0.0f);
#endif

	// Calculate final weights and make sure to "normalize" them. If the weights are too small, force them to zero, signalling an invalid sample.
	float4 FinalWeights = BilinearWeights * BilateralWeights;
	const float WeightsSum = FinalWeights.x + FinalWeights.y + FinalWeights.z + FinalWeights.w;
	FinalWeights *= WeightsSum > 1e-5f ? rcp(WeightsSum) : 0.0f;

	OutSceneDepth = dot(SceneDepth4, FinalWeights);
	OutDeviceZ = ConvertToDeviceZ(OutSceneDepth);
	OutWeights = FinalWeights;
#if SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE
	OutRefractionMask = RefractionMask4;
#else
	OutRefractionMask = 1.0f;
#endif
}
