// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SingleLayerWaterComposite - final step of the single layer water
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"  
#include "BRDF.ush"
#include "ReflectionEnvironmentShared.ush"
#include "ShadingModels.ush"
#include "SceneTextureParameters.ush"
#include "LightGridCommon.ush"
#include "/Engine/Shared/SingleLayerWaterDefinitions.h"
#define SINGLE_LAYER_WATER_USE_REFRACTION_MASK 0
#include "SingleLayerWaterCommon.ush"

#define ENABLE_SKY_LIGHT 1

#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES (FEATURE_LEVEL >= FEATURE_LEVEL_SM5)
#define REFLECTION_COMPOSITE_SUPPORT_SKYLIGHT_BLEND 0
#include "ReflectionEnvironmentComposite.ush"
#include "Substrate/SubstrateEvaluation.ush"

#if GENERATE_FROXELS


#define FROXEL_HASH_BUFFER_SIZE 64
#define FROXEL_HASH_THREAD_GROUP_SIZE 64
#define FROXEL_SHARED_HASH_BUFFER_VAR GroupShared_FroxelHashBuffer

groupshared uint GroupShared_FroxelHashBuffer[FROXEL_HASH_BUFFER_SIZE];

#include "Froxel/FroxelBuild.ush"


#endif

#ifndef THREADGROUP_SIZE
	#define THREADGROUP_SIZE 1
#endif

Texture2D ScreenSpaceReflectionsTexture;
SamplerState ScreenSpaceReflectionsSampler;

Texture2D SceneNoWaterDepthTexture;
SamplerState SceneNoWaterDepthSampler;

Texture2D<float4> SeparatedMainDirLightTexture;

float2 SceneNoWaterTextureSize;
float2 SceneNoWaterInvTextureSize;
float UseSeparatedMainDirLightTexture;

struct SingleLayerWaterCompositeOutput
{
	float4 LuminanceTransmittance;
	float  Clip;
};

SingleLayerWaterCompositeOutput SingleLayerWaterComposite(float2 BufferUV, float2 ScreenPosition, float4 SvPosition)
{
	const uint2 PixelPos = uint2(SvPosition.xy);

	// No AO or DFAO

	const float3 OneThird3 = float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f);
	const bool CameraIsUnderWater = false;

	SingleLayerWaterCompositeOutput Output;
	Output.LuminanceTransmittance = float4(0.0f, 0.0f, 0.0f, 1.0f);
	Output.Clip = 1.0f; // no clipping

	float3 TransmittanceToScene = 1.0f;
	float3 SeparatedMainDirLightContribution = 0.0f;

	const float4 SceneNoWaterMinMaxUV = GetSingleLayerWaterRefractionUVMinMax(SceneNoWaterTextureSize, SceneNoWaterInvTextureSize);

#if SUBSTRATE_GBUFFER_FORMAT==1

	// Water do nto read the classification data to avoid running it a second time when water is enabled.
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(SvPosition.xy, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	if (SubstratePixelHeader.ClosureCount > 0 && SubstratePixelHeader.IsSingleLayerWater())
	{
		float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
		float SceneDepth = ConvertFromDeviceZ(DeviceZ);
		float WaterDepth = SceneDepth;
		const float2 SceneNoWaterUV = clamp(BufferUV, SceneNoWaterMinMaxUV.xy, SceneNoWaterMinMaxUV.zw);

		float OpaqueDeviceZ;
		float OpaqueDepth;
		float4 WeightsDummy;
		float4 RefractionMaskDummy;
		SampleSceneDepthWithoutWaterBilateral(SceneNoWaterDepthTexture, SceneNoWaterDepthSampler, SceneNoWaterUV, SceneNoWaterTextureSize, SceneNoWaterInvTextureSize, WaterDepth, OpaqueDeviceZ, OpaqueDepth, WeightsDummy, RefractionMaskDummy);

		float DeltaDepth = CameraIsUnderWater ? WaterDepth : OpaqueDepth - WaterDepth;	// Inverted depth
		if (DeltaDepth > 0.0)
		{
			const FSubstrateBSDF SLWaterBSDF = UnpackSubstrateBSDFIn(Substrate.MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
			const float3 WorldNormal = SubstrateGetBSDFSharedBasis(SubstratePixelHeader, SLWaterBSDF, SubstrateAddressing)[2];
			const float3 F0 = ComputeF0(SLW_SPECULAR(SLWaterBSDF), SLW_BASECOLOR(SLWaterBSDF), SLW_METALLIC(SLWaterBSDF));
			const float Roughness = SLW_ROUGHNESS(SLWaterBSDF);
			const float SafeRoughness = MakeRoughnessSafe(Roughness);

			if (UseSeparatedMainDirLightTexture > 0.0f)
			{
				// Also compose the separated main direction light luminance (separated to be able to combine it with DistanceFieldShadow)
				SeparatedMainDirLightContribution = SeparatedMainDirLightTexture[PixelPos].rgb;
			}

			// Compute the sky reflection contribution
			float3 Reflection = 0.0f;

			// TODO MAKE COMMON WITH ReflectionEnvironment?
			float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, WaterDepth), WaterDepth, 1), View.ScreenToTranslatedWorld).xyz;
			float3 CameraToPixel = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
			float3 ReflectionVector = reflect(CameraToPixel, WorldNormal);
			float IndirectIrradiance = 0.0;
			float IndirectSpecularOcclusion = 1.0f;
			float3 ExtraIndirectSpecular = 0;
			
			//support reflection captures
			#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
				float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;

				uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), WaterDepth, 0);

				FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);
				uint NumCulledReflectionCaptures = CulledReflectionCapturesGridHeader.NumReflectionCaptures;
				uint CaptureDataStartIndex = CulledReflectionCapturesGridHeader.DataStartIndex;
			#else
				uint NumCulledReflectionCaptures = 0;
				uint CaptureDataStartIndex = 0;
			#endif

			//
			float3 N = WorldNormal;
			float3 V = -CameraToPixel;
			float3 R = 2 * dot(V, N) * N - V;
			float NoV = saturate(dot(N, V));
			// Point lobe in off-specular peak direction
			R = GetOffSpecularPeakReflectionDir(N, R, SafeRoughness);
			const bool bCompositeSkylight = true;
			Reflection += View.PreExposure * CompositeReflectionCapturesAndSkylightTWS(
				1,
				TranslatedWorldPosition,
				R,
				SafeRoughness,
				IndirectIrradiance,
				IndirectSpecularOcclusion,
				ExtraIndirectSpecular,
				NumCulledReflectionCaptures,
				CaptureDataStartIndex,
				0,
				bCompositeSkylight);

			// Then combine reflection with SSR
			float4 SSR = Texture2DSample(ScreenSpaceReflectionsTexture, ScreenSpaceReflectionsSampler, BufferUV);
			Reflection = SSR.rgb + Reflection * (1 - SSR.a);

			const float3 BSDFCoverage = 1.0f;	// Since the SLW BSDF is always isolated and composited onto the Substrate material buffer, this is assumed to be 1 here.

			// The specular over the water in completely in the hand of the user. We do not fade out metalness for instance.
			float3 EnvBrdfValue = BSDFCoverage * EnvBRDF(F0, SafeRoughness, saturate(NoV));

#else // SUBSTRATE_GBUFFER_FORMAT==1

	// Sample scene textures.
	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);
	uint ShadingModelID = GBuffer.ShadingModelID;

	if (ShadingModelID == SHADINGMODELID_SINGLELAYERWATER)
	{
		float WaterDepth = GBuffer.Depth;
		const float2 SceneNoWaterUV = clamp(BufferUV, SceneNoWaterMinMaxUV.xy, SceneNoWaterMinMaxUV.zw);

		float OpaqueDeviceZ;
		float OpaqueDepth;
		float4 WeightsDummy;
		float4 RefractionMaskDummy;
		SampleSceneDepthWithoutWaterBilateral(SceneNoWaterDepthTexture, SceneNoWaterDepthSampler, SceneNoWaterUV, SceneNoWaterTextureSize, SceneNoWaterInvTextureSize, WaterDepth, OpaqueDeviceZ, OpaqueDepth, WeightsDummy, RefractionMaskDummy);

		float DeltaDepth = CameraIsUnderWater ? WaterDepth : OpaqueDepth - WaterDepth;	// Inverted depth
		if (DeltaDepth > 0.0)
		{
			// Compute the sky reflection contribution
			float3 Reflection = 0.0f;

			// TODO MAKE COMMON WITH ReflectionEnvironment?
			const float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, GBuffer.Depth), GBuffer.Depth, 1), View.ScreenToTranslatedWorld).xyz;
			float IndirectIrradiance = 0.0;// GBuffer.IndirectIrradiance;
			float IndirectSpecularOcclusion = 1.0f;
			float3 ExtraIndirectSpecular = 0;

			//support reflection captures
			#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
				float2 LocalPosition = SvPosition.xy - View.ViewRectMin.xy;

				uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), WaterDepth, 0);

				FCulledReflectionCapturesGridHeader CulledReflectionCapturesGridHeader = GetCulledReflectionCapturesGridHeader(GridIndex);
				uint NumCulledReflectionCaptures = CulledReflectionCapturesGridHeader.NumReflectionCaptures;
				uint CaptureDataStartIndex = CulledReflectionCapturesGridHeader.DataStartIndex;
			#else
				uint NumCulledReflectionCaptures = 0;
				uint CaptureDataStartIndex = 0;
			#endif

			//
			float3 N = GBuffer.WorldNormal;
			float3 V = -GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
			float3 R = 2 * dot(V, N) * N - V;
			float NoV = saturate(dot(N, V));
			// Point lobe in off-specular peak direction
			R = GetOffSpecularPeakReflectionDir(N, R, GBuffer.Roughness);
			const bool bCompositeSkylight = true;
			Reflection += View.PreExposure * CompositeReflectionCapturesAndSkylightTWS(
				1,
				TranslatedWorldPosition,
				R,
				GBuffer.Roughness,
				IndirectIrradiance,
				IndirectSpecularOcclusion,
				ExtraIndirectSpecular,
				NumCulledReflectionCaptures,
				CaptureDataStartIndex,
				0,
				bCompositeSkylight);

			// Then combine reflection with SSR
			float4 SSR = Texture2DSample(ScreenSpaceReflectionsTexture, ScreenSpaceReflectionsSampler, BufferUV);
			Reflection = SSR.rgb + Reflection * (1 - SSR.a);

			// Apply the BRDF reflection integral to reflection. 
			// BRDF/Fresnel is already applied for the under water part in the scene.
			float3 EnvBrdfValue = EnvBRDF(GBuffer.SpecularColor, GBuffer.Roughness, NoV);

			if (UseSeparatedMainDirLightTexture > 0.0f)
			{
				// Also compose the separated main direction light luminance (separated to be able to combine it with DistanceFieldShadow)
				SeparatedMainDirLightContribution = SeparatedMainDirLightTexture[PixelPos].rgb;
			}

#endif // SUBSTRATE_GBUFFER_FORMAT==1 

			Reflection *= EnvBrdfValue;

			// Soft fading near shor to avoid seeing triangles.
			const float ShoreOpacity = saturate(DeltaDepth * 0.02f);
			Reflection.rgb *= ShoreOpacity;

			// Apply transmittance to reflection if under water
			if (CameraIsUnderWater)
			{
				TransmittanceToScene *= 1.0 - EnvBrdfValue;

				// Using default under water material: compute transmittance and scattering
				float3 WaterMediumScattering = 0.0f;
				float3 WaterMediumTransmittance = float3(0.1, 0.1, 0.8);

				Reflection *= WaterMediumTransmittance;
				TransmittanceToScene *= WaterMediumTransmittance;
			}

			Output.LuminanceTransmittance = float4(Reflection + SeparatedMainDirLightContribution, dot(OneThird3, TransmittanceToScene));
		}
		else
		{
			Output.Clip = -1.0f;
		}
	}
	else
	{
		Output.Clip = -1.0f;
	}

	return Output;
}



#if TILE_CATERGORISATION_SHADER

#if USE_WATER_PRE_PASS_STENCIL
Texture2D<uint2> WaterDepthStencilTexture;
Texture2D<float> WaterDepthTexture;
#endif

int2 TiledViewRes;
RWStructuredBuffer<uint> TileMaskBufferOut;

#if COMPILER_SUPPORTS_WAVE_VOTE
groupshared uint bAnyWaterPixels;
#else
groupshared bool ContainsWater[SLW_TILE_SIZE_XY * SLW_TILE_SIZE_XY];
#endif

bool DoesPixelContainWater(uint2 PixelPos)
{
#if USE_WATER_PRE_PASS_STENCIL
	uint Stencil = WaterDepthStencilTexture.Load(uint3(PixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;
	return (Stencil & 1U) != 0U;
#else // !USE_WATER_PRE_PASS_STENCIL
#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	return (SubstratePixelHeader.ClosureCount > 0 && SubstratePixelHeader.IsSingleLayerWater());
#else
	float2 BufferUV = (PixelPos + 0.5f) * View.BufferSizeAndInvSize.zw;
	FGBufferData GBuffer = GetGBufferDataFromSceneTextures(BufferUV);
	return (GBuffer.ShadingModelID == SHADINGMODELID_SINGLELAYERWATER);
#endif
#endif // USE_WATER_PRE_PASS_STENCIL
}

/**
 * Check all GBuffer pixels and set 1 bit for any 8x8 tile which contains a water pixel
 */
[numthreads(SLW_TILE_SIZE_XY, SLW_TILE_SIZE_XY, 1)]
void WaterTileCatergorisationMarkCS(uint3 ThreadId : SV_DispatchThreadID, uint3 GroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	const uint GroupThreadIndex = GroupThreadId.y * SLW_TILE_SIZE_XY + GroupThreadId.x;
#if 0
	// Slow reference path
	if (GroupThreadIndex < 1u)
	{
		bool bContainsWater = false;
		for (uint i = 0; i < SLW_TILE_SIZE_XY; ++i)
		{
			for (uint j = 0; j < SLW_TILE_SIZE_XY; ++j)
			{
				bContainsWater = bContainsWater || DoesPixelContainWater((ThreadId.xy + uint2(i, j)) + View.ViewRectMin.xy);
			}
		}
		if (bContainsWater)
		{
			uint WriteToIndex;
			InterlockedAdd(DrawIndirectDataUAV[1], 1, WriteToIndex);
			InterlockedAdd(DispatchIndirectDataUAV[0], 1);
			// Encoding needs to match Lumen reflection tile encoding (see LumenReflection.usf)
			WaterTileListDataUAV[WriteToIndex] = PackTileCoord12bits(GroupId.xy);
		}
	}

#else

	uint2 PixelPos = ThreadId.xy + View.ViewRectMin.xy;
	bool bContainsWater = false;
	if(ThreadId.x < uint(View.BufferSizeAndInvSize.x) && ThreadId.y < uint(View.BufferSizeAndInvSize.y))
	{
		bContainsWater = DoesPixelContainWater(PixelPos);
	}

	bool bWriteTile = false;

#if COMPILER_SUPPORTS_WAVE_VOTE

	if (all(GroupThreadId == 0))
	{
		bAnyWaterPixels = 0;
	}
	GroupMemoryBarrierWithGroupSync();

	const bool bAnyWaterPixelsInWave = WaveActiveAnyTrue(bContainsWater);
	if (WaveIsFirstLane() && bAnyWaterPixelsInWave)
	{
		InterlockedAdd(bAnyWaterPixels, 1);
	}
	GroupMemoryBarrierWithGroupSync();
	
	if (all(GroupThreadId == 0))
	{
		bWriteTile = bAnyWaterPixels > 0;
	}

#else

	ContainsWater[GroupThreadIndex] = bContainsWater;
	GroupMemoryBarrierWithGroupSync();

	if (GroupThreadIndex < 32) // 8*8 = 64 elements to merge
	{
		ContainsWater[GroupThreadIndex] = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupThreadIndex < 16)
	{
		ContainsWater[GroupThreadIndex] = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	// The smallest wave size is 16 on Intel hardware. So now we can do simple math operations without group sync.
	// EDIT: for some reason group sync is needed until the end, otherwise some pixels are missing...

	if (GroupThreadIndex < 8)
	{
		ContainsWater[GroupThreadIndex] = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupThreadIndex < 4)
	{
		ContainsWater[GroupThreadIndex] = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupThreadIndex < 2)
	{
		ContainsWater[GroupThreadIndex] = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();
	if (GroupThreadIndex < 1)
	{
		bWriteTile = ContainsWater[GroupThreadIndex] || ContainsWater[GroupThreadIndex + 1];
	}

#endif

	if (bWriteTile)
	{
		// Set bit to indicate tile is occupied.
		uint MaskLinearIndex = TiledViewRes.x * GroupId.y + GroupId.x;
		InterlockedOr(TileMaskBufferOut[MaskLinearIndex / 32U], 1U << (MaskLinearIndex % 32U));
	}
#endif

#if GENERATE_FROXELS
	// Need to get value that's valid for the whole group
#if COMPILER_SUPPORTS_WAVE_VOTE
	if (bAnyWaterPixels != 0)
#endif
	{
		float DeviceZ = 0.0f;
		BRANCH
		if (bContainsWater)
		{
			DeviceZ = WaterDepthTexture.Load(uint3(PixelPos, 0)).x;
		}
		// One group per 8x8 tile
		const uint2 GroupTileOffset = GroupId.xy;
		// Only one 8x8 tile per group
		const uint LocalLinearTileId = 0;
		HashBuildFroxelsFromDeviceZ(DeviceZ, GroupThreadIndex, LocalLinearTileId, GroupTileOffset);
	}
#endif

} 

int2 FullTiledViewRes;
uint VertexCountPerInstanceIndirect;
RWBuffer<uint> DrawIndirectDataUAV;
RWBuffer<uint> DispatchIndirectDataUAV;
RWBuffer<uint> DispatchClearIndirectDataUAV;
RWBuffer<uint> WaterTileListDataUAV;
RWBuffer<uint> ClearTileListDataUAV;
StructuredBuffer<uint> TileMaskBuffer;

groupshared uint SharedNumTiles;
groupshared uint SharedNumClearTiles;
groupshared uint SharedTileData[THREADGROUP_SIZE * THREADGROUP_SIZE];
groupshared uint SharedGlobalTileOffset;
groupshared uint SharedGlobalClearTileOffset;

#ifdef WaterTileClassificationBuildListsCS

/**
 * Every group checks 64 tiles and builds a spatially coherent compacted list of water tiles
 */
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)] 
void WaterTileClassificationBuildListsCS(
	uint2 GroupId : SV_GroupID,
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupThreadId : SV_GroupThreadID)
{
	const uint LinearThreadIndex = GroupThreadId.y * THREADGROUP_SIZE + GroupThreadId.x;

	if (all(DispatchThreadId == 0))
	{
	#if DOWNSAMPLE_FACTOR_X == 1 && DOWNSAMPLE_FACTOR_Y == 1
		// TODO compute clear myself
		DrawIndirectDataUAV[0] = VertexCountPerInstanceIndirect;		// VertexCountPerInstance
		//DrawIndirectDataUAV[1]		// InstanceCount			already cleared to 0
		//DrawIndirectDataUAV[2] = 0;	// StartVertexLocation			"			"
		//DrawIndirectDataUAV[3] = 0;	// StartInstanceLocation		"			"
	#endif

		DispatchIndirectDataUAV[1] = 1;
		DispatchIndirectDataUAV[2] = 1;
	#if OUTPUT_CLEAR_TILES
		DispatchClearIndirectDataUAV[1] = 1;
		DispatchClearIndirectDataUAV[2] = 1;
	#endif
	}

#if WAVE_OPS

	GroupMemoryBarrierWithGroupSync();

	// ZOrder tiles to maximize screen locality after converting to 1d for compaction
	// The tile locality ultimately affects trace coherency, since trace compaction pulls from neighboring tiles
	uint2 ThreadOffset = ZOrder2D(LinearThreadIndex, log2(THREADGROUP_SIZE));
	uint2 TileCoordinate = GroupId * THREADGROUP_SIZE + ThreadOffset;

	if (all(TileCoordinate < TiledViewRes))
	{
		uint2 FullTileCoord00 = TileCoordinate * uint2(DOWNSAMPLE_FACTOR_X, DOWNSAMPLE_FACTOR_Y);
		bool bTileUsed = false;

		for (uint OffsetY = 0; OffsetY < DOWNSAMPLE_FACTOR_Y; ++OffsetY)
		{
			uint2 FullTileCoord = FullTileCoord00 + uint2(0, OffsetY);

			if (FullTileCoord.y < FullTiledViewRes.y)
			{
				uint MaskLinearIndex = FullTiledViewRes.x * FullTileCoord.y + FullTileCoord.x;
				uint Mask = 1u << (MaskLinearIndex % 32u);
				uint MaskDword = TileMaskBuffer[MaskLinearIndex / 32u];
			#if DOWNSAMPLE_FACTOR_X == 2
				Mask |= select(FullTileCoord.x + 1 < FullTiledViewRes.x, Mask << 1u, Mask);
			#endif
				bTileUsed = bTileUsed || ((MaskDword & Mask) != 0);

				if (bTileUsed)
				{
					break;
				}
			}
		}

		// Active tiles
		{
			uint NumTilesInWave = WaveActiveCountBits(bTileUsed);
			uint GlobalTileOffset = 0;
			if (WaveIsFirstLane() && NumTilesInWave > 0)
			{
			#if DOWNSAMPLE_FACTOR_X == 1 && DOWNSAMPLE_FACTOR_Y == 1
				InterlockedAdd(DrawIndirectDataUAV[1], NumTilesInWave);
			#endif
				InterlockedAdd(DispatchIndirectDataUAV[0], NumTilesInWave, GlobalTileOffset);
			}

			GlobalTileOffset = WaveReadLaneFirst(GlobalTileOffset);

			if (bTileUsed)
			{
				// Encoding needs to match Lumen reflection tile encoding (see LumenReflection.usf)
				WaterTileListDataUAV[GlobalTileOffset + WavePrefixCountBits(true)] = PackTileCoord12bits(TileCoordinate);
			}
		}

		// Unused tiles
		#if OUTPUT_CLEAR_TILES
		{
			uint NumTilesInWave = WaveActiveCountBits(!bTileUsed);
			uint GlobalTileOffset = 0;
			if (WaveIsFirstLane() && NumTilesInWave > 0)
			{
				InterlockedAdd(DispatchClearIndirectDataUAV[0], NumTilesInWave, GlobalTileOffset);
			}

			GlobalTileOffset = WaveReadLaneFirst(GlobalTileOffset);

			if (!bTileUsed)
			{
				// Note: Must match encoding in WaterTileCatergorisationBuildListsCS
				ClearTileListDataUAV[GlobalTileOffset + WavePrefixCountBits(true)] = PackTileCoord12bits(TileCoordinate);
			}
		}
		#endif
	}

#else // !WAVE_OPS

	if (LinearThreadIndex == 0)
	{
		SharedNumTiles = 0;
	#if OUTPUT_CLEAR_TILES
		SharedNumClearTiles = 0;
	#endif
	}

	GroupMemoryBarrierWithGroupSync();

	//@todo - parallel version
	if (LinearThreadIndex == 0)
	{
		SharedNumTiles = 0;
	#if OUTPUT_CLEAR_TILES
		SharedNumClearTiles = 0;
	#endif

		for (uint LocalTileIndex = 0; LocalTileIndex < THREADGROUP_SIZE * THREADGROUP_SIZE; ++LocalTileIndex)
		{
			// ZOrder tiles to maximize screen locality after converting to 1d for compaction
			// The tile locality ultimately affects trace coherency, since trace compaction pulls from neighboring tiles
			uint2 ThreadOffset = ZOrder2D(LocalTileIndex, log2(THREADGROUP_SIZE));
			uint2 TileCoordinate = GroupId * THREADGROUP_SIZE + ThreadOffset;

			if (all(TileCoordinate < TiledViewRes))
			{
				uint2 FullTileCoord00 = TileCoordinate * uint2(DOWNSAMPLE_FACTOR_X, DOWNSAMPLE_FACTOR_Y);
				bool bTileUsed = false;

				for (uint OffsetY = 0; OffsetY < DOWNSAMPLE_FACTOR_Y; ++OffsetY)
				{
					uint2 FullTileCoord = FullTileCoord00 + uint2(0, OffsetY);

					if (FullTileCoord.y < FullTiledViewRes.y)
					{
						uint MaskLinearIndex = FullTiledViewRes.x * FullTileCoord.y + FullTileCoord.x;
						uint Mask = 1u << (MaskLinearIndex % 32u);
						uint MaskDword = TileMaskBuffer[MaskLinearIndex / 32u];
					#if DOWNSAMPLE_FACTOR_X == 2
						Mask |= select(FullTileCoord.x + 1 < FullTiledViewRes.x, Mask << 1u, Mask);
					#endif
						bTileUsed = bTileUsed || ((MaskDword & Mask) != 0);

						if (bTileUsed)
						{
							break;
						}
					}
				}

				if (bTileUsed)
				{
					uint TileOffset = SharedNumTiles;
					// Encoding needs to match Lumen reflection tile encoding (see LumenReflection.usf)
					SharedTileData[TileOffset] = PackTileCoord12bits(TileCoordinate);
					SharedNumTiles = TileOffset + 1;
				}
				else
				{
				#if OUTPUT_CLEAR_TILES
					// Pack clear tiles from the other end
					uint TileOffset = SharedNumClearTiles;
					SharedTileData[THREADGROUP_SIZE * THREADGROUP_SIZE - 1 - TileOffset] = PackTileCoord12bits(TileCoordinate);
					SharedNumClearTiles = TileOffset + 1;
				#endif
				}
			}
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Allocate space in the tile list
	if (LinearThreadIndex == 0 && SharedNumTiles > 0)
	{
	#if DOWNSAMPLE_FACTOR_X == 1 && DOWNSAMPLE_FACTOR_Y == 1
		InterlockedAdd(DrawIndirectDataUAV[1], SharedNumTiles);
	#endif
		InterlockedAdd(DispatchIndirectDataUAV[0], SharedNumTiles, SharedGlobalTileOffset);
	}

#if OUTPUT_CLEAR_TILES
	if (LinearThreadIndex == 0 && SharedNumClearTiles > 0)
	{
		InterlockedAdd(DispatchClearIndirectDataUAV[0], SharedNumClearTiles, SharedGlobalClearTileOffset);
	}
#endif

	GroupMemoryBarrierWithGroupSync();

	// Write out tiles
	if (LinearThreadIndex < SharedNumTiles)
	{
		WaterTileListDataUAV[SharedGlobalTileOffset + LinearThreadIndex] = SharedTileData[LinearThreadIndex];
	}
	else
	{
	#if OUTPUT_CLEAR_TILES
		uint LocalThreadIndex = LinearThreadIndex - SharedNumTiles;
		if (LocalThreadIndex < SharedNumClearTiles)
		{
			ClearTileListDataUAV[SharedGlobalClearTileOffset + LocalThreadIndex] = SharedTileData[THREADGROUP_SIZE * THREADGROUP_SIZE - 1 - LocalThreadIndex];
		}
	#endif
	}
#endif
}

#endif //WaterTileClassificationBuildListsCS

#elif defined(TILE_VERTEX_SHADER)


Buffer<uint> TileListData;

void WaterTileVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION) 
{
	const uint PackedTile = TileListData[InstanceId.x];
	const uint2 TileCoord = UnpackTileCoord12bits(PackedTile);
	const uint2 TileOrigin = TileCoord * SLW_TILE_SIZE_XY;

	uint2 TileVertex = TileOrigin;
	TileVertex.x += VertexId == 1 || VertexId == 2 || VertexId == 4 ? SLW_TILE_SIZE_XY : 0;
	TileVertex.y += VertexId == 2 || VertexId == 4 || VertexId == 5 ? SLW_TILE_SIZE_XY : 0;

	// View port is set on the view rect. So no offset are needed.
	Position = float4(float2(TileVertex) * View.ViewSizeAndInvSize.zw * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);
}


#else



void SingleLayerWaterCompositePS(
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0)
{
	ResolvedView = ResolveView();

	float2 BufferUV = SvPositionToBufferUV(SvPosition);
	float2 ScreenPosition = SvPositionToScreenPosition(SvPosition).xy;

	// TODO use dual source blending to apply TransmittanceToScene
	SingleLayerWaterCompositeOutput Result = SingleLayerWaterComposite(BufferUV, ScreenPosition, SvPosition);

	clip(Result.Clip); // Since this shader does not write to depth or stencil it should still benefit from EarlyZ (See AMD depth-in-depth documentation)
	OutColor = Result.LuminanceTransmittance;
}



#endif

Texture2D<float4> SceneColorCopyDownsampleTexture;
SamplerState      SceneColorCopyDownsampleSampler;
Texture2D<float4> SceneDepthCopyDownsampleTexture;
SamplerState      SceneDepthCopyDownsampleSampler;
float2 SVPositionToSourceTextureUV;

void WaterRefractionCopyPS(
	in float4 SVPosition : SV_Position
#if COPY_DEPTH
	, out float OutSceneDepth : SV_Target0
#elif COPY_COLOR
	, out float4 OutSceneColor : SV_Target0
#endif
#if COPY_DEPTH && COPY_COLOR
	, out float4 OutSceneColor : SV_Target1
#endif
)
{
	float2 BufferUV = SVPosition.xy * SVPositionToSourceTextureUV;

#if COPY_DEPTH
	#if DOWNSAMPLE_REFRACTION
		// Pick furthermost depth to minimize downsampling artifacts (UE4 is using reverse-z)
		float4 Depth4 = SceneDepthCopyDownsampleTexture.Gather(SceneDepthCopyDownsampleSampler, BufferUV);
		float DeviceDepthZ = min(min(min(Depth4.x, Depth4.y), Depth4.z), Depth4.w);
	#else
		float DeviceDepthZ = Texture2DSampleLevel(SceneDepthCopyDownsampleTexture, SceneDepthCopyDownsampleSampler, BufferUV, 0).x;
	#endif
	OutSceneDepth = DeviceDepthZ;
#endif

#if COPY_COLOR
	#if DOWNSAMPLE_REFRACTION || !SUPPORTS_INDEPENDENT_SAMPLERS
		// Downsample and average all 2x2 quads
		OutSceneColor.xyz = Texture2DSampleLevel(SceneColorCopyDownsampleTexture, SceneColorCopyDownsampleSampler, BufferUV, 0).xyz;
	#else
		// Use the depth point sampler when not downsampling color
		OutSceneColor.xyz = Texture2DSampleLevel(SceneColorCopyDownsampleTexture, SceneDepthCopyDownsampleSampler, BufferUV, 0).xyz;
	#endif

	OutSceneColor.w = 1.0f;
#endif
}

