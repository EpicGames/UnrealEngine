// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Common.ush"
#include "SingleLayerWaterCommon.ush"

#ifdef SingleLayerWaterRefractionMaskPS

#include "BRDF.ush"

Texture2D<float> SceneDepthTexture;
Texture2D<float> WaterDepthTexture;
float DistanceCullingRangeBegin;
float DistanceCullingRangeEnd;
float FresnelCullingRangeBegin;
float FresnelCullingRangeEnd;
float DepthCullingRangeBegin;
float DepthCullingRangeEnd;

void SingleLayerWaterRefractionMaskPS(
	in float4 SvPosition : SV_Position,
	out float OutRefractionMask : SV_Target0
)
{
	const float WaterDeviceZ = WaterDepthTexture.Load(int3(SvPosition.xy, 0)).x;
	const float3 WaterTranslatedWorldPos = SvPositionToTranslatedWorld(float4(SvPosition.xy, WaterDeviceZ, 1.0f));
	const float3 CameraTranslatedWorldPos = GetTranslatedWorldCameraPosFromView(SvPosition.xy);
	const float3 CameraToWater = WaterTranslatedWorldPos - CameraTranslatedWorldPos;
	
	float ResultMask = 1.0f;

	if (DistanceCullingRangeEnd > 0.0f)
	{
		const float Distance2 = length2(CameraToWater);
		const float Distance = Distance2 > 1e-5f ? sqrt(Distance2) : 0.0f;
		ResultMask *= 1.0f - smoothstep(DistanceCullingRangeBegin, DistanceCullingRangeEnd, Distance);
	}

	if (FresnelCullingRangeEnd > 0.0f)
	{
		const float3 WorldPosDDX = ddx_fine(WaterTranslatedWorldPos);
		const float3 WorldPosDDY = ddy_fine(WaterTranslatedWorldPos);
		const float3 WorldNormal = normalize(cross(WorldPosDDX, WorldPosDDY));

		const float3 V = normalize(-CameraToWater);
		const float3 L = reflect(-V, WorldNormal);
		const float3 H = normalize(V + L);
		const float Fresnel = F_Schlick(0.04f, abs(dot(V, H))).g;

		ResultMask *= 1.0f - smoothstep(FresnelCullingRangeEnd, FresnelCullingRangeBegin, Fresnel);
	}

	if (DepthCullingRangeEnd > 0.0f)
	{
		const float LinearWaterDepth = ConvertFromDeviceZ(WaterDeviceZ);
		const float LinearSceneDepth = ConvertFromDeviceZ(SceneDepthTexture.Load(int3(SvPosition.xy, 0)).x);
		const float DepthDiff = LinearSceneDepth - LinearWaterDepth;

		ResultMask *= 1.0f - smoothstep(DepthCullingRangeBegin, DepthCullingRangeEnd, DepthDiff);
	}
	
	OutRefractionMask = PackWaterRefractionMask(true /*bIsWater*/, ResultMask);
}

#endif //SingleLayerWaterRefractionMaskPS

#ifdef SingleLayerWaterRefractionCullingPS

Texture2D<float> WaterRefractionCullingTexture;
Texture2D<float> WaterDepthTexture;

void SingleLayerWaterRefractionCullingPS(
	in float4 SvPosition : SV_Position,
	out float OutDepth : SV_Depth
#if SHADING_MASK_EXPORT
	, out uint OutShadingMask : SV_Target0
#endif
)
{
	const float PackedRefractionMask = WaterRefractionCullingTexture.Load(int3(SvPosition.xy, 0)).x;
	bool bIsWater = false;
	float RefractionMask = 0.0f;
	UnpackWaterRefractionMask(PackedRefractionMask, bIsWater, RefractionMask);
	if (!bIsWater || RefractionMask > 0.0f)
	{
		discard;
	}

	// Always a water pixel at this stage
	const float WaterDeviceZ = WaterDepthTexture.Load(int3(SvPosition.xy, 0)).x;
	OutDepth = WaterDeviceZ;

#if SHADING_MASK_EXPORT
	OutShadingMask = 0;
#endif
}

#endif // SingleLayerWaterRefractionCullingPS