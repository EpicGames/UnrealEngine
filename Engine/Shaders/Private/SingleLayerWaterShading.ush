// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "ParticipatingMediaCommon.ush" 
#include "ShadingCommon.ush" 
#include "DistortionCommon.ush"
#include "SingleLayerWaterCommon.ush"

#if SINGLE_LAYER_WATER_SHADING_QUALITY != SINGLE_LAYER_WATER_SHADING_QUALITY_FULL && SINGLE_LAYER_WATER_SHADING_QUALITY != SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE && SINGLE_LAYER_WATER_SHADING_QUALITY != SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_BUFFER
#error SINGLE_LAYER_WATER_SHADING_QUALITY has an invalid value!
#endif

// Helper define for all shading qualities using mobile shading
#define SINGLE_LAYER_WATER_MOBILE_SHADING (SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE) || (SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_BUFFER) 

#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	// We need the water base pass to be allowed to use masking (if blend mode is Masked) because it doesn't rely on a depth pre-pass (where clipping would normally occur)
	#undef EARLY_Z_PASS_ONLY_MATERIAL_MASKING
	#define EARLY_Z_PASS_ONLY_MATERIAL_MASKING 0
#endif

float3 SampleSceneColorWithoutWaterBilateral(Texture2D Tex, SamplerState TexSampler, float2 UV, float4 Weights)
{
	const float4 Red = Tex.GatherRed(TexSampler, UV);
	const float4 Green = Tex.GatherGreen(TexSampler, UV);
	const float4 Blue = Tex.GatherBlue(TexSampler, UV);

	float3 BehindWaterSceneLuminance = float3(dot(Red, Weights), dot(Green, Weights), dot(Blue, Weights));



	return BehindWaterSceneLuminance;
}

struct WaterVolumeLightingOutput
{
	float3 Luminance;
	float3 WaterToSceneTransmittance;
	float3 WaterToSceneToLightTransmittance;
};

// Returns as luminance, the result of lighting scattering in water and the under water scene color contribution.
WaterVolumeLightingOutput EvaluateWaterVolumeLighting(
	FMaterialPixelParameters MaterialParameters, FPixelMaterialInputs PixelMaterialInputs, ViewState ResolvedView,
	float DirectionalLightShadow,
#if SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_FULL || SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE
	Texture2D SceneDepthWithoutSingleLayerWaterTexture, SamplerState SceneDepthWithoutSingleLayerWaterSampler,
	float2 SceneDepthWithoutSingleLayerWaterTextureSize,
	float2 SceneDepthWithoutSingleLayerWaterTextureInvSize,
#endif
#if SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_FULL
	Texture2D SceneColorWithoutSingleLayerWaterTexture, SamplerState SceneColorWithoutSingleLayerWaterSampler,
	float2 SceneWithoutSingleLayerWaterMinUV,
	float2 SceneWithoutSingleLayerWaterMaxUV,
	#if SINGLE_LAYER_WATER_USE_REFRACTION_MASK
	Texture2D RefractionMaskTexture,
	#endif
#endif
	float Specular, const float4 DistortionParams,
	float3 SunIlluminance, float3 AmbiantIlluminance, float3 EnvBrdf,
	bool bCameraIsUnderWater, float WaterVisibility, uint EyeIndex,
	bool bSeparateMainDirLight, inout float3 SeparatedWaterMainDirLightScatteredLuminance,
	in float3 LightFunctionColor = 1.0f)
{
	WaterVolumeLightingOutput Output;
	Output.Luminance = 0.0f;
	Output.WaterToSceneTransmittance = 1.0f;
	Output.WaterToSceneToLightTransmittance = 1.0f;

	float2 ViewportUV = MaterialParameters.ViewBufferUV;

	if (WaterVisibility > 0.0f)
	{
		float WaterSurfaceSceneDepth = GetPixelDepth(MaterialParameters); // Scene depth with Pixel-Depth-Offset taken into account is in ScreenPosition.w already
#if SINGLE_LAYER_WATER_MOBILE_SHADING

#if SINGLE_LAYER_WATER_SHADING_QUALITY == SINGLE_LAYER_WATER_SHADING_QUALITY_MOBILE_WITH_DEPTH_TEXTURE 
		float BehindWaterSceneDeviceZ;
		float BehindWaterSceneDepth;
		float4 BehindWaterSceneColorWeights;
		float4 DummyRefractionMask;
		SampleSceneDepthWithoutWaterBilateral(SceneDepthWithoutSingleLayerWaterTexture, SceneDepthWithoutSingleLayerWaterSampler, ViewportUV, SceneDepthWithoutSingleLayerWaterTextureSize, SceneDepthWithoutSingleLayerWaterTextureInvSize,
			WaterSurfaceSceneDepth,
			BehindWaterSceneDeviceZ, BehindWaterSceneDepth, BehindWaterSceneColorWeights, DummyRefractionMask);
#else
		float BehindWaterSceneDeviceZ = LookupDeviceZ(ViewportUV);
		float BehindWaterSceneDepth = ConvertFromDeviceZ(BehindWaterSceneDeviceZ);
#endif

		float2 BehindWaterUV = ViewportUV;

#else  // !SINGLE_LAYER_WATER_MOBILE_SHADING

		float IorWater = DielectricF0ToIor(DielectricSpecularToF0(Specular)); // Wrong if metal is set to >1. But we still keep refraction on the water surface nonetheless.

		ViewportUV = clamp(ViewportUV, SceneWithoutSingleLayerWaterMinUV, SceneWithoutSingleLayerWaterMaxUV);

		// Sample SceneDepthWithoutWater at current pixel location
		float PixelBehindWaterSceneDeviceZ;
		float PixelBehindWaterSceneDepth;
		float4 PixelSceneColorWeights;
		float4 PixelRefractionMask;
		SampleSceneDepthWithoutWaterBilateral(SceneDepthWithoutSingleLayerWaterTexture, SceneDepthWithoutSingleLayerWaterSampler, ViewportUV, SceneDepthWithoutSingleLayerWaterTextureSize, SceneDepthWithoutSingleLayerWaterTextureInvSize,
			WaterSurfaceSceneDepth,
		#if SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE
			RefractionMaskTexture,
		#endif
			PixelBehindWaterSceneDeviceZ, PixelBehindWaterSceneDepth, PixelSceneColorWeights, PixelRefractionMask);

		// Compute a distorted/refracted UV
		FMaterialRefractionData RefractionData = GetMaterialRefraction(PixelMaterialInputs);
		float2 BufferUVDistortion = ComputeBufferUVDistortion(
			MaterialParameters, PixelMaterialInputs, ResolvedView,
			MaterialParameters.WorldNormal, IorWater,
			DistortionParams, ViewportUV, RefractionData, false, EyeIndex);
		PostProcessUVDistortion(MaterialParameters, PixelMaterialInputs, PixelBehindWaterSceneDepth, BufferUVDistortion, RefractionData);
		// Also reduce refraction according to water thickness
		BufferUVDistortion *= saturate((PixelBehindWaterSceneDepth - WaterSurfaceSceneDepth) * 1.0 / 30.0f);// *(1 - saturate(WaterDepth * 1.0 / 2000.0f));
		const float2 DistortedUV = clamp(ViewportUV + BufferUVDistortion, SceneWithoutSingleLayerWaterMinUV, SceneWithoutSingleLayerWaterMaxUV);

		// Sample SceneDepthWithoutWater at distorted UV
		float DistortedSceneDeviceZ;
		float DistortedSceneDepth;
		float4 DistortedSceneColorWeights;
		float4 DistortedRefractionMask;
		SampleSceneDepthWithoutWaterBilateral(SceneDepthWithoutSingleLayerWaterTexture, SceneDepthWithoutSingleLayerWaterSampler, DistortedUV, SceneDepthWithoutSingleLayerWaterTextureSize, SceneDepthWithoutSingleLayerWaterTextureInvSize,
			WaterSurfaceSceneDepth,
#if SINGLE_LAYER_WATER_REFRACTION_MASK_AVAILABLE
			RefractionMaskTexture,
#endif
			DistortedSceneDeviceZ, DistortedSceneDepth, DistortedSceneColorWeights, DistortedRefractionMask);

		// Pick a BehindWaterDepth depending on whether the weights of the refracted location are valid
		float BehindWaterSceneDepth;
		float BehindWaterSceneDeviceZ;
		float4 BehindWaterSceneColorWeights;
		float2 BehindWaterUV;
		if (all(DistortedSceneColorWeights == 0.0f))
		{
			BehindWaterSceneDepth = PixelBehindWaterSceneDepth;
			BehindWaterSceneDeviceZ = PixelBehindWaterSceneDeviceZ;
			BehindWaterSceneColorWeights = PixelSceneColorWeights * PixelRefractionMask;
			BehindWaterUV = ViewportUV;
		}
		else
		{
			BehindWaterSceneDepth = DistortedSceneDepth;
			BehindWaterSceneDeviceZ = DistortedSceneDeviceZ;
			BehindWaterSceneColorWeights = DistortedSceneColorWeights * DistortedRefractionMask;
			BehindWaterUV = DistortedUV;
		}

#endif // !SINGLE_LAYER_WATER_MOBILE_SHADING

		// SvPositionToWorld will result in NaN if SceneDeviceZ=0
		BehindWaterSceneDeviceZ = max(0.000000000001f, BehindWaterSceneDeviceZ);

		const float WaterVolumeDepth = bCameraIsUnderWater ? WaterSurfaceSceneDepth : max(0.0f, BehindWaterSceneDepth - WaterSurfaceSceneDepth);

#if SUBSTRATE_INLINE_SINGLELAYERWATER
		MaterialParameters.SubstrateTree.BSDFs[0].SubstrateSanitizeBSDF();
		FSubstrateBSDF SLWBSDF = MaterialParameters.SubstrateTree.BSDFs[0];
		const float3 Albedo = SLW_WATERALBEDO(SLWBSDF);
		const float3 ExtinctionCoeff = SLW_WATEREXTINCTION(SLWBSDF);
		const float3 ScatteringCoeff = Albedo * ExtinctionCoeff;
		const float3 AbsorptionCoeff = max(0.0, ExtinctionCoeff - ScatteringCoeff);
		const float PhaseG = SLW_WATERPHASEG(SLWBSDF);
		//Sample the optional Material Input ColorScaleBehindWater and fade it out at shorelines to avoid hard edge intersections
		float3 ColorScaleBehindWater = lerp(1.0f, SLW_COLORSCALEBEHINDWATER(SLWBSDF), saturate(WaterVolumeDepth * 0.02f));
#else
		const float3 ScatteringCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput0(MaterialParameters)));
		const float3 AbsorptionCoeff = max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput1(MaterialParameters)));
		const float PhaseG = clamp(DFDemote(GetSingleLayerWaterMaterialOutput2(MaterialParameters).x), -1.0f, 1.0f);
		//Sample the optional Material Input ColorScaleBehindWater and fade it out at shorelines to avoid hard edge intersections
		float3 ColorScaleBehindWater = lerp(1.0f, max(0.0f, DFDemote(GetSingleLayerWaterMaterialOutput3(MaterialParameters))), saturate(WaterVolumeDepth * 0.02f));
		const float3 ExtinctionCoeff = ScatteringCoeff + AbsorptionCoeff;
		const float3 Albedo = ScatteringCoeff / max(ExtinctionCoeff, 1e-7f);
#endif
		
		// Max to avoid division by 0 with the analytical integral below.
		// 1e-5 is high enough to avoid denorms on mobile
		const float3 ExtinctionCoeffSafe = max(ExtinctionCoeff, 1e-5);
		
		float DirLightPhaseValue = 0.0f; // Default when Total Internal Reflection happens.
		{
#if SINGLE_LAYER_WATER_MOBILE_SHADING
			DirLightPhaseValue = IsotropicPhase();
#else
			float IorFrom = 1.0f; // assumes we come from air
			const float relativeIOR = IorFrom / IorWater;
			float3 UnderWaterRayDir = 0.0f;
			if (WaterRefract(MaterialParameters.CameraVector, MaterialParameters.WorldNormal, relativeIOR, UnderWaterRayDir))
			{
				DirLightPhaseValue = SchlickPhase(PhaseG, dot(-ResolvedView.DirectionalLightDirection.xyz, UnderWaterRayDir));
			}
#endif
		}

		// We also apply transmittance from light to under water surface. However, the scene has been lit by many sources already.
		// So the transmittance toabove surface is simply approximated using the travel distance from the scene pixel to the water top, assuming a flat water surface.
		// We cannot combine this transmittance with the transmittance from view because this would change the behavior of the analytical integration of light scattering integration.
		const float3 BehindWaterSceneWorldPos = SvPositionToResolvedTranslatedWorld(float4(MaterialParameters.SvPosition.xy, BehindWaterSceneDeviceZ, 1.0));
		const float DistanceFromScenePixelToWaterTop = max(0.0, (GetTranslatedWorldPosition(MaterialParameters).z - BehindWaterSceneWorldPos.z));
		const float3 MeanTransmittanceToLightSources = exp(-DistanceFromScenePixelToWaterTop * ExtinctionCoeff);

#if SINGLE_LAYER_WATER_MOBILE_SHADING
		const float3 BehindWaterSceneLuminance = 0.0f; // Cannot read back the scene color in this case
#else
		float3 BehindWaterSceneLuminance = SampleSceneColorWithoutWaterBilateral(SceneColorWithoutSingleLayerWaterTexture, SceneColorWithoutSingleLayerWaterSampler, BehindWaterUV, BehindWaterSceneColorWeights);
		BehindWaterSceneLuminance *= MeanTransmittanceToLightSources * ResolvedView.OneOverPreExposure;
#endif

		float3 SunScattLuminance = DirLightPhaseValue * SunIlluminance;
		float3 AmbScattLuminance = IsotropicPhase()   * AmbiantIlluminance;

		const float MainDirLightFactor			= bSeparateMainDirLight ? 0.0f : 1.0f;
		const float SeparatedMainDirLightFactor = bSeparateMainDirLight ? 1.0f : 0.0f;

		const float3 OpticalDepth = ExtinctionCoeff * WaterVolumeDepth;
		float3 Transmittance = exp(-OpticalDepth);
		float3 IncomingLuminance = (AmbScattLuminance + SunScattLuminance * LightFunctionColor * DirectionalLightShadow * MainDirLightFactor);
		// The SafeScatteringAmount should be in 0~1, add saturate to fix an artifact caused by half precision on mobile.
		float3 SafeScatteringAmount = saturate(ScatteringCoeff * (1.0f - Transmittance) / ExtinctionCoeffSafe);
		float3 ScatteredLuminance = IncomingLuminance * SafeScatteringAmount;

		// Compute separated direcitonal light scattering if requested
		float3 SeparatedIncomingLuminance = SunScattLuminance * LightFunctionColor * DirectionalLightShadow * SeparatedMainDirLightFactor;
		// Apply Fresnel and water visibility to separated main dir light luminance.
		SeparatedWaterMainDirLightScatteredLuminance += SeparatedIncomingLuminance * SafeScatteringAmount * (bCameraIsUnderWater ? 1.0 : (1.0 - EnvBrdf)) * WaterVisibility;

		// Apply Fresnel effect to out-scattering towards the view
		ScatteredLuminance *= bCameraIsUnderWater ? 1.0 : (1.0 - EnvBrdf);	// Under water is less visible due to Fresnel effect
		Transmittance *= bCameraIsUnderWater ? (1.0 - EnvBrdf) : 1.0;		// Above	"		"		"		"		"

		// Add single in-scattering apply colored transmittance to scene color
		Output.Luminance = WaterVisibility * (ScatteredLuminance + Transmittance * (BehindWaterSceneLuminance * ColorScaleBehindWater));
		Output.WaterToSceneTransmittance = Transmittance;
		Output.WaterToSceneToLightTransmittance = Transmittance * MeanTransmittanceToLightSources;
	}

	return Output;
}

#undef SINGLE_LAYER_WATER_MOBILE_SHADING
