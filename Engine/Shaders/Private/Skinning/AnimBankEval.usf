// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "/Engine/Private/Visualization.ush"
#include "/Engine/Private/Skinning/SkinningDataDecode.ush"
#include "/Engine/Shared/SkinningDefinitions.h"

ByteAddressBuffer SequenceTransformBuffer;
ByteAddressBuffer BankBlockBuffer;
ByteAddressBuffer BoneMapBuffer;
FCompressedBoneTransformRWBuffer TransformBuffer;

float SampleInterval;
uint bInterpolating;

struct FKeyIndices
{
	int KeyIndex1;
	int KeyIndex2;
	float Alpha;
};

FKeyIndices GetKeyIndicesFromTime(float Time, int KeyCount, float SequenceLength, float FramesPerSecond)
{
	FKeyIndices Indices;

	// Check for 1-frame, before-first-frame and after-last-frame cases.
	if (Time <= 0.0f || KeyCount == 1)
	{
		Indices.KeyIndex1 = 0;
		Indices.KeyIndex2 = 0;
		Indices.Alpha = 0.0f;
	}
	else if (Time >= SequenceLength)
	{
		Indices.KeyIndex1 = KeyCount - 1;
		Indices.KeyIndex2 = 0;
		Indices.Alpha = 0.0f;
	}
	else
	{
		// Calculate the frames per second if we didn't provide any.
		if (FramesPerSecond <= 0.0f)
		{
			const int FrameCount = KeyCount - 1;
			FramesPerSecond = FrameCount / SequenceLength;
		}

		const float KeyPos = Time * FramesPerSecond;

		// Find the integer part (ensuring within range) and that gives us the 'starting' key index.
		int IntegerPart = int(KeyPos);
		IntegerPart -= ((float)IntegerPart > KeyPos);
		Indices.KeyIndex1 = clamp(IntegerPart, 0, KeyCount - 1);

		// The alpha (fractional part) is just the remainder.
		Indices.Alpha = KeyPos - (float)Indices.KeyIndex1;

		Indices.KeyIndex2 = Indices.KeyIndex1 + 1;

		// If we have gone over the end, do different things in case of looping
		if (Indices.KeyIndex2 == KeyCount)
		{
			Indices.KeyIndex2 = Indices.KeyIndex1;
		}
	}

	return Indices;
}

[numthreads(BONES_PER_GROUP, 1, 1)]
void BankEvaluateCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint BoneBlock           = GroupId.x;
	const uint LocalTransformIndex = GroupIndex;

	const FAnimBankBlockHeader BoneBlockHeader = BankBlockBuffer.Load<FAnimBankBlockHeader>(BoneBlock * uint(sizeof(FAnimBankBlockHeader)));
	if (LocalTransformIndex < BoneBlockHeader.BlockTransformCount)
	{
		const uint BoneTransformSize      = uint(sizeof(FBoneTransform));
		const uint MappingTransformIndex  = BoneBlockHeader.MappingTransformIndex;
		const uint SequenceTransformIndex = BoneBlockHeader.SequenceTransformIndex;
		const uint TransformIndex         = LocalTransformIndex + (BoneBlockHeader.BlockLocalIndex * BONES_PER_GROUP);

		uint BoneIndex = TransformIndex;
	
		if (BoneBlockHeader.BoneMapOffset != ~uint(0))
		{
			BoneIndex = BoneMapBuffer.Load<uint>(BoneBlockHeader.BoneMapOffset + TransformIndex * uint(sizeof(uint)));
		}

		const float TrackLength = (BoneBlockHeader.FrameCount - 1) * SampleInterval;
		const float Time = BoneBlockHeader.Time;

		const float FramesPerSecond = -1.0f;
		FKeyIndices KeyIndices = GetKeyIndicesFromTime(Time, BoneBlockHeader.FrameCount, TrackLength, FramesPerSecond);

		if (!bInterpolating)
		{
			// Forcing alpha to zero disables pose interpolation (interpolation method is "step")
			KeyIndices.Alpha = 0.0f;
		}

		// TODO: Scale version

		FBoneTransform InvGlobalRefPoseTransform = SequenceTransformBuffer.Load<FBoneTransform>((MappingTransformIndex + BoneIndex) * BoneTransformSize);
		FBoneTransform MeshGlobalAnimPoseTransform;

		if (KeyIndices.Alpha <= 0.0f)
		{
			const uint KeyIndex = (KeyIndices.KeyIndex1 * BoneBlockHeader.BoneCount) + BoneIndex;
			MeshGlobalAnimPoseTransform = SequenceTransformBuffer.Load<FBoneTransform>((SequenceTransformIndex + KeyIndex) * BoneTransformSize);
			QuatNormalize(MeshGlobalAnimPoseTransform.Rotation);
		}
		else if (KeyIndices.Alpha >= 1.0f)
		{
			const uint KeyIndex = (KeyIndices.KeyIndex2 * BoneBlockHeader.BoneCount) + BoneIndex;
			MeshGlobalAnimPoseTransform = SequenceTransformBuffer.Load<FBoneTransform>((SequenceTransformIndex + KeyIndex) * BoneTransformSize);
			QuatNormalize(MeshGlobalAnimPoseTransform.Rotation);
		}
		else
		{
			const uint KeyIndexA = (KeyIndices.KeyIndex1 * BoneBlockHeader.BoneCount) + BoneIndex;
			const uint KeyIndexB = (KeyIndices.KeyIndex2 * BoneBlockHeader.BoneCount) + BoneIndex;

			FBoneTransform MeshGlobalTransformA = SequenceTransformBuffer.Load<FBoneTransform>((SequenceTransformIndex + KeyIndexA) * BoneTransformSize);
			FBoneTransform MeshGlobalTransformB = SequenceTransformBuffer.Load<FBoneTransform>((SequenceTransformIndex + KeyIndexB) * BoneTransformSize);

			// Ensure rotations are normalized before blending
			QuatNormalize(MeshGlobalTransformA.Rotation);
			QuatNormalize(MeshGlobalTransformB.Rotation);

			// NOTE: Rotation will already be normalized
			MeshGlobalAnimPoseTransform = BlendBoneTransforms(MeshGlobalTransformA, MeshGlobalTransformB, KeyIndices.Alpha);
		}

		MeshGlobalAnimPoseTransform = MultiplyBoneTransform(InvGlobalRefPoseTransform, MeshGlobalAnimPoseTransform);
		const float4x3 CurrentTransform = (float4x3)MatrixFromBoneTransform(MeshGlobalAnimPoseTransform);

		const uint OutputTransformOffset = BoneBlockHeader.OutputTransformIndex * uint(sizeof(FCompressedBoneTransform));
		StoreCompressedBoneTransform(TransformBuffer, OutputTransformOffset, TransformIndex, CurrentTransform);

		if (BoneBlockHeader.OutputTransformCopyIndexOffset != 0)
		{
			const uint SecondaryOutputTransformOffset = (BoneBlockHeader.OutputTransformIndex + BoneBlockHeader.OutputTransformCopyIndexOffset) * uint(sizeof(FCompressedBoneTransform));
			StoreCompressedBoneTransform(TransformBuffer, SecondaryOutputTransformOffset, TransformIndex, CurrentTransform);
		}
	}
}