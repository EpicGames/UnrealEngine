// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "/Engine/Private/Visualization.ush"
#include "/Engine/Private/Skinning/SkinningDataDecode.ush"
#include "/Engine/Shared/SkinningDefinitions.h"

// Allow register spilling for complex mesh-space permutations.
#pragma warning(disable:7203)

ByteAddressBuffer RetargetingDataBuffer;
ByteAddressBuffer BoneMapBuffer;
StructuredBuffer<float4> SequenceTransformBuffer;
StructuredBuffer<FEvaluateLayerHeader> LayerHeaderBuffer;
StructuredBuffer<FSamplePoseData> SamplePoseBuffer;
FCompressedBoneTransformRWBuffer SkinningTransformBufferUAV;
uint NumTotalHeaders;
uint HeaderBaseOffset;
uint SlotsPerInstance;
uint NumCompactBones;
uint MeshSpaceMask;
uint LayerModeMask;
uint NumLayers;

groupshared FBoneTransformWithUniformScale SharedTransform[BONES_PER_GROUP];

#define RETARGET_MODE_ANIMATION 0
#define RETARGET_MODE_SKELETON 1
#define RETARGET_MODE_ANIMATION_SCALED 2
#define RETARGET_MODE_ANIMATION_RELATIVE 3
#define RETARGET_MODE_ORIENT_AND_SCALE 4

#define INVALID_BONE_INDEX 0xFFFF

FBoneRetargetingData LoadBoneRetargetingData(ByteAddressBuffer Buffer, uint BaseOffset, uint Index)
{
	uint Offset = BaseOffset + Index * (uint)sizeof(FBoneRetargetingData);

	return Buffer.Load<FBoneRetargetingData>(Offset);
}

FBoneTransformWithUniformScale LoadBoneTransformWithUniformScale(StructuredBuffer<float4> Buffer, uint BaseIndex, uint Index)
{
	FBoneTransformWithUniformScale Output;

#if HALF_PRECISION_SEQUENCES
	float4 Packed = Buffer[BaseIndex + Index];
	Output.Rotation = FQuat(
		f16tof32(asuint(Packed.x)),
		f16tof32(asuint(Packed.x) >> 16),
		f16tof32(asuint(Packed.y)),
		f16tof32(asuint(Packed.y) >> 16));
	Output.Location = float3(
		f16tof32(asuint(Packed.z)),
		f16tof32(asuint(Packed.z) >> 16),
		f16tof32(asuint(Packed.w)));
	Output.Scale = f16tof32(asuint(Packed.w) >> 16);
#else
	const uint Float4Index = BaseIndex + Index * 2;
	float4 D0 = Buffer[Float4Index];
	float4 D1 = Buffer[Float4Index + 1];
	Output.Rotation = FQuat(D0.x, D0.y, D0.z, D0.w);
	Output.Location = float3(D1.x, D1.y, D1.z);
	Output.Scale    = D1.w;
#endif

	return Output;
}

void StoreBoneTransformWithUniformScale(RWByteAddressBuffer Buffer, uint BaseOffset, uint Index, FBoneTransformWithUniformScale Transform)
{
	uint Offset = BaseOffset + Index * (uint)sizeof(FBoneTransformWithUniformScale);

	Buffer.Store4(Offset, asuint(float4(Transform.Rotation)));
	Buffer.Store4(Offset + 16, asuint(float4(Transform.Location, Transform.Scale.x)));
}

FBoneTransformWithUniformScale EvaluateAndRetargetBone(
	uint BoneIndex,
	uint SequenceTransformBufferOffset,
	uint RetargetingTransformBufferOffset,
	uint RetargetingDataBufferOffset,
	uint KeyIndex0,
	uint KeyIndex1,
	float Alpha,
	uint NumAnimBones)
{
	FBoneRetargetingData RetargetingData = LoadBoneRetargetingData(RetargetingDataBuffer, RetargetingDataBufferOffset, BoneIndex);

	const uint RetargetingTransformIndex = BoneIndex * (RetargetingData.bRequiresRefPoseRetarget ? 2 : 1);
	const bool bIsRootBone               = BoneIndex == 0;

	FBoneTransformWithUniformScale RetargetingTransform = LoadBoneTransformWithUniformScale(SequenceTransformBuffer, RetargetingTransformBufferOffset, RetargetingTransformIndex);

	if (RetargetingData.AnimBoneIndex == INVALID_BONE_INDEX)
	{
		// When the anim bone doesn't exist the ref pose is stored in the retargeting transform.
		return RetargetingTransform;
	}

	const uint KeyIndexA = RetargetingData.AnimBoneIndex + KeyIndex0 * NumAnimBones;
	const uint KeyIndexB = RetargetingData.AnimBoneIndex + KeyIndex1 * NumAnimBones;

	FBoneTransformWithUniformScale MeshLocalTransformA    = LoadBoneTransformWithUniformScale(SequenceTransformBuffer, SequenceTransformBufferOffset, KeyIndexA);
	FBoneTransformWithUniformScale MeshLocalTransformB    = LoadBoneTransformWithUniformScale(SequenceTransformBuffer, SequenceTransformBufferOffset, KeyIndexB);
	FBoneTransformWithUniformScale MeshLocalAnimTransform = BlendBoneTransformsWithUniformScale(MeshLocalTransformA, MeshLocalTransformB, Alpha);

	if (RetargetingData.bRequiresRefPoseRetarget && !bIsRootBone)
	{
		FBoneTransformWithUniformScale RefPoseData = LoadBoneTransformWithUniformScale(SequenceTransformBuffer, RetargetingTransformBufferOffset, RetargetingTransformIndex + 1);

		// Unpack Q0 from Rotation
		FQuat Q0 = FQuat(RefPoseData.Rotation[0], RefPoseData.Rotation[1], RefPoseData.Rotation[2], RefPoseData.Rotation[3]);

		// Unpack Q1 from Location.xyz + Scale.x
		FQuat Q1 = FQuat(RefPoseData.Location.x, RefPoseData.Location.y, RefPoseData.Location.z, RefPoseData.Scale);

		MeshLocalAnimTransform.Rotation = QuatNormalize(QuatMultiply(QuatMultiply(Q0, MeshLocalAnimTransform.Rotation), Q1));

		if (RetargetingData.RetargetMode != RETARGET_MODE_SKELETON)
		{
			MeshLocalAnimTransform.Location = QuatRotateVector(Q0, MeshLocalAnimTransform.Location);
		}
	}

	switch (RetargetingData.RetargetMode)
	{
		case RETARGET_MODE_ANIMATION:
			break;

		case RETARGET_MODE_SKELETON:
			MeshLocalAnimTransform.Location = RetargetingTransform.Location;
			if (bIsRootBone)
			{
				MeshLocalAnimTransform.Rotation = RetargetingTransform.Rotation;
			}
			break;

		case RETARGET_MODE_ANIMATION_RELATIVE:
			MeshLocalAnimTransform.Rotation = QuatNormalize(QuatMultiply(MeshLocalAnimTransform.Rotation, RetargetingTransform.Rotation));
			MeshLocalAnimTransform.Location += RetargetingTransform.Location;
			MeshLocalAnimTransform.Scale *= RetargetingTransform.Scale;
			break;

		case RETARGET_MODE_ANIMATION_SCALED:
			MeshLocalAnimTransform.Location *= RetargetingTransform.Scale;
			break;

		case RETARGET_MODE_ORIENT_AND_SCALE:
			MeshLocalAnimTransform.Location = QuatRotateVector(RetargetingTransform.Rotation, MeshLocalAnimTransform.Location) * RetargetingTransform.Scale.x;
			break;
	}

	return MeshLocalAnimTransform;
}

// SharedTransform .Scale field packing:
// Lower 16 bits = FFloat16 value (bone scale or mesh-space Q.w)
// Upper 16 bits = ParentIndex (for chain walk without BoneMapBuffer reads)
float PackScaleWithParent(float Value, uint ParentIndex)
{
	return asfloat((f32tof16(Value) & 0xFFFF) | (ParentIndex << 16));
}

float UnpackScale(float Packed)
{
	return f16tof32(asuint(Packed) & 0xFFFF);
}

uint UnpackParentIndex(float Packed)
{
	return asuint(Packed) >> 16;
}

void StoreSharedTransform(uint Index, FBoneTransformWithUniformScale Transform, uint ParentIndex)
{
	Transform.Scale = PackScaleWithParent(Transform.Scale, ParentIndex);
	SharedTransform[Index] = Transform;
}

FBoneTransformWithUniformScale LoadSharedTransform(uint Index)
{
	FBoneTransformWithUniformScale T = SharedTransform[Index];
	T.Scale = UnpackScale(T.Scale);
	return T;
}

uint GetSharedParentIndex(uint Index)
{
	return UnpackParentIndex(SharedTransform[Index].Scale);
}

FBoneHierarchyData LoadBoneHierarchyData(ByteAddressBuffer Buffer, uint BaseOffset, uint Index)
{
	uint Offset = BaseOffset + Index * (uint)sizeof(FBoneHierarchyData);
	return Buffer.Load<FBoneHierarchyData>(Offset);
}

ByteAddressBuffer BoneMaskBuffer;

float LoadBoneMaskWeight(FEvaluateLayerHeader Header, uint BoneIndex)
{
	if (!HasBoneMask(Header))
	{
		return 1.0f;
	}

	uint BoneMaskOffset = GetBoneMaskOffset(Header);
	return asfloat(BoneMaskBuffer.Load(BoneMaskOffset + BoneIndex * 4));
}

FBoneTransformWithUniformScale EvaluateWeightedSamples(uint BoneIndex, uint Offset, uint Count)
{
	FSamplePoseData Sample = SamplePoseBuffer[Offset];
	FBoneTransformWithUniformScale Result = EvaluateAndRetargetBone(
		BoneIndex,
		Sample.SequenceTransformBufferOffset,
		Sample.RetargetingTransformBufferOffset,
		Sample.RetargetingDataBufferOffset,
		Sample.KeyIndex0, Sample.KeyIndex1,
		Sample.Alpha, Sample.NumAnimBones);
	float AccumulatedWeight = Sample.Weight;

	if (Count >= 2)
	{
		Sample = SamplePoseBuffer[Offset + 1];
		FBoneTransformWithUniformScale Pose = EvaluateAndRetargetBone(
			BoneIndex,
			Sample.SequenceTransformBufferOffset,
			Sample.RetargetingTransformBufferOffset,
			Sample.RetargetingDataBufferOffset,
			Sample.KeyIndex0, Sample.KeyIndex1,
			Sample.Alpha, Sample.NumAnimBones);
		float SampleWeight = Sample.Weight;
		Result = BlendBoneTransformsWithUniformScale(Result, Pose, SampleWeight / (AccumulatedWeight + SampleWeight));
		AccumulatedWeight += SampleWeight;
	}

	if (Count >= 3)
	{
		Sample = SamplePoseBuffer[Offset + 2];
		FBoneTransformWithUniformScale Pose = EvaluateAndRetargetBone(
			BoneIndex,
			Sample.SequenceTransformBufferOffset,
			Sample.RetargetingTransformBufferOffset,
			Sample.RetargetingDataBufferOffset,
			Sample.KeyIndex0, Sample.KeyIndex1,
			Sample.Alpha, Sample.NumAnimBones);
		float SampleWeight = Sample.Weight;
		Result = BlendBoneTransformsWithUniformScale(Result, Pose, SampleWeight / (AccumulatedWeight + SampleWeight));
	}

	return Result;
}

FBoneTransformWithUniformScale ConcatenateToGlobal(
	uint BoneIndex,
	uint SharedTransformOffset,
	uint InvGlobalRefPoseTransformBufferOffset)
{
	FBoneTransformWithUniformScale GlobalTransform = LoadSharedTransform(SharedTransformOffset + BoneIndex);

	uint ParentIndex = GetSharedParentIndex(SharedTransformOffset + BoneIndex);
	while (ParentIndex != INVALID_BONE_INDEX)
	{
		GlobalTransform = MultiplyBoneTransformWithUniformScale(GlobalTransform, LoadSharedTransform(SharedTransformOffset + ParentIndex));
		ParentIndex = GetSharedParentIndex(SharedTransformOffset + ParentIndex);
	}

	FBoneTransformWithUniformScale InvRefPose = LoadBoneTransformWithUniformScale(SequenceTransformBuffer, InvGlobalRefPoseTransformBufferOffset, BoneIndex);
	return MultiplyBoneTransformWithUniformScale(InvRefPose, GlobalTransform);
}

void ScatterGlobalSpaceTransform(uint DstTransformBufferOffset1, uint DstTransformBufferOffset2, uint TransformIndex, FBoneTransformWithUniformScale GlobalTransform)
{
	float4x3 OutputMatrix = (float4x3)MatrixFromBoneTransformWithUniformScale(GlobalTransform);
	StoreCompressedBoneTransform(SkinningTransformBufferUAV, DstTransformBufferOffset1, TransformIndex, OutputMatrix);

	if (DstTransformBufferOffset1 != DstTransformBufferOffset2)
	{
		StoreCompressedBoneTransform(SkinningTransformBufferUAV, DstTransformBufferOffset2, TransformIndex, OutputMatrix);
	}
}

#if HAS_MESH_SPACE_LAYERS

void StoreMeshSpaceLayerRotation(uint Index, FQuat Q, uint ParentIndex)
{
	SharedTransform[Index].Location = float3(Q.x, Q.y, Q.z);
	SharedTransform[Index].Scale = PackScaleWithParent(Q.w, ParentIndex);
}

FQuat LoadMeshSpaceLayerRotation(uint Index)
{
	float3 XYZ = SharedTransform[Index].Location;
	float W = UnpackScale(SharedTransform[Index].Scale);
	return FQuat(XYZ.x, XYZ.y, XYZ.z, W);
}

struct FMeshSpaceBlendResult
{
	FQuat BlendedGlobal;
	uint ParentIndex;
	float3 Location;
	float Scale;
};

FMeshSpaceBlendResult BlendMeshSpaceOverride(
	FBoneTransformWithUniformScale LayerPose,
	FBoneTransformWithUniformScale Result,
	uint BoneIndex,
	uint SharedOffset,
	float EffectiveWeight)
{
	uint SharedIndex        = SharedOffset + BoneIndex;
	FQuat BaseLocal         = SharedTransform[SharedIndex].Rotation;
	FQuat LayerLocal        = LoadMeshSpaceLayerRotation(SharedIndex);
	FQuat BaseParentGlobal  = QuatIdentity();
	FQuat LayerParentGlobal = QuatIdentity();

	uint ParentIndex = GetSharedParentIndex(SharedIndex);

	uint WalkIndex = ParentIndex;
	while (WalkIndex != INVALID_BONE_INDEX)
	{
		uint ParentSharedIndex = SharedOffset + WalkIndex;
		BaseParentGlobal       = QuatMultiply(SharedTransform[ParentSharedIndex].Rotation, BaseParentGlobal);
		LayerParentGlobal      = QuatMultiply(LoadMeshSpaceLayerRotation(ParentSharedIndex), LayerParentGlobal);
		WalkIndex              = GetSharedParentIndex(ParentSharedIndex);
	}

	FQuat BaseGlobal    = QuatMultiply(BaseParentGlobal, BaseLocal);
	FQuat LayerGlobal   = QuatMultiply(LayerParentGlobal, LayerLocal);
	FQuat BlendedGlobal = QuatNormalize(QuatSlerpApproximate(BaseGlobal, LayerGlobal, EffectiveWeight));

	FMeshSpaceBlendResult Out;
	Out.BlendedGlobal = BlendedGlobal;
	Out.ParentIndex   = ParentIndex;
	Out.Location      = lerp(Result.Location, LayerPose.Location, EffectiveWeight);
	Out.Scale         = lerp(Result.Scale, LayerPose.Scale, EffectiveWeight);
	return Out;
}

FMeshSpaceBlendResult BlendMeshSpaceAdditive(
	FBoneTransformWithUniformScale LayerPose,
	FBoneTransformWithUniformScale Result,
	uint BoneIndex,
	uint SharedOffset,
	float EffectiveWeight)
{
	uint SharedIndex = SharedOffset + BoneIndex;
	FQuat BaseGlobal       = SharedTransform[SharedIndex].Rotation;
	FQuat BaseParentGlobal = QuatIdentity();

	uint ParentIndex = GetSharedParentIndex(SharedIndex);

	uint WalkIndex = ParentIndex;
	while (WalkIndex != INVALID_BONE_INDEX)
	{
		BaseParentGlobal = QuatMultiply(SharedTransform[SharedOffset + WalkIndex].Rotation, BaseParentGlobal);
		WalkIndex = GetSharedParentIndex(SharedOffset + WalkIndex);
	}

	BaseGlobal = QuatMultiply(BaseParentGlobal, BaseGlobal);

	FQuat WeightedDelta = QuatSlerpApproximate(QuatIdentity(), LayerPose.Rotation, EffectiveWeight);
	FQuat BlendedGlobal = QuatNormalize(QuatMultiply(WeightedDelta, BaseGlobal));

	FMeshSpaceBlendResult Out;
	Out.BlendedGlobal = BlendedGlobal;
	Out.ParentIndex   = ParentIndex;
	Out.Location      = Result.Location + LayerPose.Location * EffectiveWeight;
	Out.Scale         = Result.Scale * lerp(1.0, LayerPose.Scale, EffectiveWeight);
	return Out;
}

#endif

[numthreads(BONES_PER_GROUP, 1, 1)]
void EvaluateConcatenateScatterCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const uint InstancesPerGroup = BONES_PER_GROUP / SlotsPerInstance;

	const uint InstanceInGroup       = GroupIndex / SlotsPerInstance;
	const uint BoneIndex             = GroupIndex % SlotsPerInstance;
	const uint HeaderIndex           = min(GroupId.x * InstancesPerGroup + InstanceInGroup, NumTotalHeaders - 1);
	const uint SharedTransformOffset = InstanceInGroup * SlotsPerInstance;

	const bool bValidInstance = HeaderIndex < NumTotalHeaders;
	const bool bActiveLane = bValidInstance && BoneIndex < NumCompactBones;
	const uint SharedIndex = SharedTransformOffset + BoneIndex;

	FBoneTransformWithUniformScale Result = MakeBoneTransformWithUniformScaleIdentity();

	uint InvGlobalRefPoseTransformBufferOffset = 0;
	uint BoneMapBufferOffset = 0;
	uint DstTransformBufferOffset1 = 0;
	uint DstTransformBufferOffset2 = 0;
	uint BoneTransformIndex = 0;

	uint BoneParentIndex = INVALID_BONE_INDEX;

#if SINGLE_LAYER
	const uint LayerCount = 1;
#else
	const uint LayerCount = NumLayers;
#endif

	for (uint LayerIndex = 0; LayerIndex < LayerCount; LayerIndex++)
	{
		FEvaluateLayerHeader Header = LayerHeaderBuffer[HeaderBaseOffset + (HeaderIndex * LayerCount) + LayerIndex];

		if (LayerIndex == 0)
		{
			BoneMapBufferOffset       = Header.BoneMapBufferOffset;
			DstTransformBufferOffset1 = Header.DstTransformBufferOffset1;
			DstTransformBufferOffset2 = Header.DstTransformBufferOffset2;
			InvGlobalRefPoseTransformBufferOffset = Header.InvGlobalRefPoseTransformBufferOffset;

			if (bActiveLane)
			{
				FBoneHierarchyData InitHierarchy = LoadBoneHierarchyData(BoneMapBuffer, BoneMapBufferOffset, BoneIndex);
				BoneParentIndex = InitHierarchy.ParentIndex;
				BoneTransformIndex = InitHierarchy.TransformIndex;
			}
		}

#if SINGLE_LAYER
		const bool  bMeshSpaceRotation = false;
		const float LayerWeight = 1.0f;
		const float EffectiveWeight = 1.0f;
#else
		const float LayerWeight        = Header.LayerWeight;
		const uint  LayerMode          = (LayerModeMask >> (LayerIndex * SKINNING_LAYER_MODE_BITS_PER_LAYER)) & SKINNING_LAYER_MODE_MASK_PER_LAYER;
		const bool  bMeshSpaceRotation = (MeshSpaceMask & (1u << LayerIndex)) != 0;
		const bool  bMeshSpaceAdditive = bMeshSpaceRotation && (LayerMode == SKINNING_LAYER_MODE_ADDITIVE);

		float EffectiveWeight = LayerWeight;
		if (EffectiveWeight > 0 && HasBoneMask(Header))
		{
			EffectiveWeight *= LoadBoneMaskWeight(Header, BoneIndex);
		}
#endif

		FBoneTransformWithUniformScale LayerPose = MakeBoneTransformWithUniformScaleIdentity();

		if (bActiveLane && (EffectiveWeight > 0 || (bMeshSpaceRotation && LayerWeight > 0)))
		{
			const uint SampleOffset = Header.SampleDataOffset;
			const uint NumSourceSamples = GetNumSourceSamples(Header);
			const uint NumTargetSamples = GetNumTargetSamples(Header);

			LayerPose = EvaluateWeightedSamples(BoneIndex, SampleOffset, NumSourceSamples);

			if (NumTargetSamples > 0 && Header.BlendWeight > 0)
			{
				FBoneTransformWithUniformScale TargetPose = EvaluateWeightedSamples(
					BoneIndex, SampleOffset + NumSourceSamples, NumTargetSamples);
				LayerPose = BlendBoneTransformsWithUniformScale(
					LayerPose, TargetPose, Header.BlendWeight);
			}

#if SINGLE_LAYER
			Result = LayerPose;
#else
			if (EffectiveWeight > 0 && !bMeshSpaceRotation)
			{
				if (LayerMode == SKINNING_LAYER_MODE_OVERRIDE)
				{
					Result = BlendBoneTransformsWithUniformScale(Result, LayerPose, EffectiveWeight);
				}
				else
				{
					Result = ApplyAdditiveLocalSpace(Result, LayerPose, EffectiveWeight);
				}
			}
#endif
		}

#if HAS_MESH_SPACE_LAYERS
		if (bMeshSpaceRotation)
		{
			if (bActiveLane && LayerWeight > 0)
			{
				StoreSharedTransform(SharedIndex, Result, BoneParentIndex);
	
				if (!bMeshSpaceAdditive)
				{
					StoreMeshSpaceLayerRotation(SharedIndex, LayerPose.Rotation, BoneParentIndex);
				}
			}

			GroupMemoryBarrierWithGroupSync();

			FMeshSpaceBlendResult MeshResult = (FMeshSpaceBlendResult)0;

			if (bActiveLane && LayerWeight > 0)
			{
				if (LayerMode == SKINNING_LAYER_MODE_OVERRIDE)
				{
					MeshResult = BlendMeshSpaceOverride(LayerPose, Result, BoneIndex, SharedTransformOffset, EffectiveWeight);
				}
				else
				{
					MeshResult = BlendMeshSpaceAdditive(LayerPose, Result, BoneIndex, SharedTransformOffset, EffectiveWeight);
				}

				StoreMeshSpaceLayerRotation(SharedIndex, MeshResult.BlendedGlobal, BoneParentIndex);
			}

			GroupMemoryBarrierWithGroupSync();

			if (bActiveLane && LayerWeight > 0)
			{
				FQuat BlendedParentGlobal = QuatIdentity();
				if (MeshResult.ParentIndex != INVALID_BONE_INDEX)
				{
					BlendedParentGlobal = LoadMeshSpaceLayerRotation(SharedTransformOffset + MeshResult.ParentIndex);
				}

				Result.Rotation = QuatNormalize(QuatMultiply(QuatConjugate(BlendedParentGlobal), MeshResult.BlendedGlobal));
				Result.Location = MeshResult.Location;
				Result.Scale    = MeshResult.Scale;
			}

			GroupMemoryBarrierWithGroupSync();
		}
#endif
	}
	
	if (bActiveLane)
	{
		StoreSharedTransform(SharedIndex, Result, BoneParentIndex);
	}
	GroupMemoryBarrierWithGroupSync();

	if (bActiveLane && BoneTransformIndex != INVALID_BONE_INDEX)
	{
		FBoneTransformWithUniformScale GlobalTransform = ConcatenateToGlobal(
			BoneIndex, SharedTransformOffset,
			InvGlobalRefPoseTransformBufferOffset);

		ScatterGlobalSpaceTransform(
			DstTransformBufferOffset1,
			DstTransformBufferOffset2,
			BoneTransformIndex, GlobalTransform);
	}
}