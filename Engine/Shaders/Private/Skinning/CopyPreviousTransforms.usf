// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Shared/SkinningDefinitions.h"

// Copies bone transforms from old buffer offsets to new offsets.
// Used when ISKM allocations move (defrag or realloc) to preserve motion vectors.
// Copies the full allocation (both current and previous slots).
//
// Two modes:
//   USE_STAGING_BUFFER=1: Read from staging SRV (defrag — old/new offsets may overlap in live buffer)
//   USE_STAGING_BUFFER=0: Read/write same UAV (realloc — allocate-before-free guarantees no overlap)

struct FCopyTransformsHeader
{
	uint SrcByteOffset;	// Byte offset in source buffer
	uint DstByteOffset;	// Byte offset in destination buffer
	uint Count;			// Number of compressed bone transforms to copy
	uint _Pad0;
};

FCompressedBoneTransformRWBuffer DstTransformBuffer;
StructuredBuffer<FCopyTransformsHeader> CopyHeaders;

#if USE_STAGING_BUFFER
FCompressedBoneTransformBuffer SrcTransformBuffer;
#endif

// Dispatch: (CeilDiv(MaxElementsAcrossAllEntries, COPY_THREAD_GROUP_SIZE), NumCopyEntries, 1)
// GroupId.x = element group index, GroupId.y = copy entry index

[numthreads(COPY_THREAD_GROUP_SIZE, 1, 1)]
void CopyPreviousTransformsCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)
{
	const FCopyTransformsHeader Header = CopyHeaders[GroupId.y];
	const uint ElementIndex = GroupId.x * COPY_THREAD_GROUP_SIZE + GroupIndex;

	if (ElementIndex >= Header.Count)
	{
		return;
	}

#if USE_COMPRESSED_BONE_TRANSFORM
	// ByteAddressBuffer: 32 bytes per transform
	const uint SrcOffset = Header.SrcByteOffset + ElementIndex * 32u;
	const uint DstOffset = Header.DstByteOffset + ElementIndex * 32u;

#if USE_STAGING_BUFFER
	const uint4 Data0 = SrcTransformBuffer.Load4(SrcOffset);
	const uint4 Data1 = SrcTransformBuffer.Load4(SrcOffset + 16u);
#else
	const uint4 Data0 = DstTransformBuffer.Load4(SrcOffset);
	const uint4 Data1 = DstTransformBuffer.Load4(SrcOffset + 16u);
#endif
	DstTransformBuffer.Store4(DstOffset, Data0);
	DstTransformBuffer.Store4(DstOffset + 16u, Data1);
#else
	// RWBuffer<float4>: 3 float4s (48 bytes) per transform
	const uint SrcBaseIndex = Header.SrcByteOffset / 16u + ElementIndex * 3u;
	const uint DstBaseIndex = Header.DstByteOffset / 16u + ElementIndex * 3u;

#if USE_STAGING_BUFFER
	DstTransformBuffer[DstBaseIndex + 0] = SrcTransformBuffer[SrcBaseIndex + 0];
	DstTransformBuffer[DstBaseIndex + 1] = SrcTransformBuffer[SrcBaseIndex + 1];
	DstTransformBuffer[DstBaseIndex + 2] = SrcTransformBuffer[SrcBaseIndex + 2];
#else
	float4 T0 = DstTransformBuffer[SrcBaseIndex + 0];
	float4 T1 = DstTransformBuffer[SrcBaseIndex + 1];
	float4 T2 = DstTransformBuffer[SrcBaseIndex + 2];
	DstTransformBuffer[DstBaseIndex + 0] = T0;
	DstTransformBuffer[DstBaseIndex + 1] = T1;
	DstTransformBuffer[DstBaseIndex + 2] = T2;
#endif
#endif
}
