// Copyright Epic Games, Inc. All Rights Reserved.

// NOTE: GPUSceneWriter.ush must be included BEFORE SceneData.ush as it defines USE_GPU_SCENE_DATA_RW.
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GPUScene/GPUSceneWriter.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "/Engine/Private/GpuSkinCommon.ush"
#include "/Engine/Private/Skinning/SkinningDataDecode.ush"
#include "/Engine/Shared/SkinningDefinitions.h"

StructuredBuffer<FResolveAttachmentBlockHeader> AttachmentBlockHeaderBuffer;
FCompressedBoneTransformBuffer SkinningTransformBuffer;
uint NumAttachments;

float4x4 LoadRefPoseMatrix(FResolveAttachmentBlockHeader Header)
{
	float4 Row0 = float4(Header.BoneRefPoseRow0[0], Header.BoneRefPoseRow0[1], Header.BoneRefPoseRow0[2], Header.BoneRefPoseRow0[3]);
	float4 Row1 = float4(Header.BoneRefPoseRow1[0], Header.BoneRefPoseRow1[1], Header.BoneRefPoseRow1[2], Header.BoneRefPoseRow1[3]);
	float4 Row2 = float4(Header.BoneRefPoseRow2[0], Header.BoneRefPoseRow2[1], Header.BoneRefPoseRow2[2], Header.BoneRefPoseRow2[3]);
	float3x4 Packed = float3x4(Row0, Row1, Row2);
	float4x3 Unpacked = transpose(Packed);
	return float4x4(float4(Unpacked[0], 0), float4(Unpacked[1], 0), float4(Unpacked[2], 0), float4(Unpacked[3], 1));
}

float4x4 LoadLocalOffsetMatrix(FResolveAttachmentBlockHeader Header)
{
	FQuat Quat = FQuat(
		Header.LocalOffsetRotation[0],
		Header.LocalOffsetRotation[1],
		Header.LocalOffsetRotation[2],
		Header.LocalOffsetRotation[3]);

	float3 Trans = float3(Header.LocalOffsetTranslation[0], Header.LocalOffsetTranslation[1], Header.LocalOffsetTranslation[2]);

	return MatrixFromBoneTransform(MakeBoneTransform(Quat, Trans));
}

// Resolve a bone from the skinning buffer, undo InvRefPose, compose with local offset and parent world transform.
// If BoneIndex is 0xFFFF (bone not in skinning buffer), uses ref pose directly (no animation).
float4x4 ComposeBoneAttachmentTransform(
	uint BoneBase,
	float4x4 RefPoseMatrix,
	float4x4 LocalOffsetMatrix,
	float4x4 ParentLocalToWorld,
	float3 PositionHigh,
	uint BoneIndex)
{
	float4x4 Bone;
	if (BoneIndex != 0xFFFF)
	{
		float4x3 SkinMatrix = LoadCompressedBoneTransform(SkinningTransformBuffer, 0, BoneBase + BoneIndex);
		Bone = mul(RefPoseMatrix, float4x4(float4(SkinMatrix[0], 0), float4(SkinMatrix[1], 0), float4(SkinMatrix[2], 0), float4(SkinMatrix[3], 1)));
	}
	else
	{
		Bone = RefPoseMatrix;
	}

	float4x4 Result = mul(mul(LocalOffsetMatrix, Bone), ParentLocalToWorld);
	Result[3].xyz -= PositionHigh;
	return Result;
}

[numthreads(64, 1, 1)]
void ResolveAttachmentsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint AttachmentIndex = DispatchThreadId.x;
	if (AttachmentIndex >= NumAttachments)
	{
		return;
	}

	FResolveAttachmentBlockHeader Header = AttachmentBlockHeaderBuffer[AttachmentIndex];

	// Unpack packed fields.
	uint ParentMaxTransformCount     = GetMaxTransformCount(Header);
	uint BoneIndex                   = GetBoneIndex(Header);
	uint ParentCurrentTransformSlot  = GetCurrentTransformSlot(Header);
	uint ParentInstanceId            = GetParentInstanceId(Header);

	FInstanceSceneData ParentInstance = GetInstanceSceneData(ParentInstanceId);
	FPrimitiveSceneData ChildPrimitive = GetPrimitiveData(Header.ChildPrimitiveId);

	float4x4 RefPoseMatrix = LoadRefPoseMatrix(Header);
	float4x4 LocalOffsetMatrix = LoadLocalOffsetMatrix(Header);

	uint CurrentBase = GetSkinningBoneTransformOffset(Header.ParentTransformBufferOffset, ParentInstance.SkinningData, ParentCurrentTransformSlot, ParentMaxTransformCount, false);
	WriteInstanceLocalToRelativeWorld(
		Header.ChildInstanceId,
		ComposeBoneAttachmentTransform(CurrentBase, RefPoseMatrix, LocalOffsetMatrix, DFDemote(ParentInstance.LocalToWorld), ChildPrimitive.PositionHigh, BoneIndex),
		true);

	uint PreviousBase = GetSkinningBoneTransformOffset(Header.ParentTransformBufferOffset, ParentInstance.SkinningData, ParentCurrentTransformSlot, ParentMaxTransformCount, true);
	WriteInstanceDynamicData(
		GetInstancePayloadDataOffsets(Header.ChildInstanceId),
		ComposeBoneAttachmentTransform(PreviousBase, RefPoseMatrix, LocalOffsetMatrix, DFDemote(ParentInstance.PrevLocalToWorld), ChildPrimitive.PositionHigh, BoneIndex));
}
