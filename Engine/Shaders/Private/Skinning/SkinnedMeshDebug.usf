// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "/Engine/Private/Visualization.ush"
#include "/Engine/Private/Skinning/SkinningSceneDataDecode.ush"

uint PersistentPrimitiveIndex;
uint bPreviousTransform;

float3 GetBoneColor(uint BoneIndex, uint BoneDepth, uint BoneCount)
{
#if 0
	float Scale = 20.0f;
	return ColorMapTurbo((float(BoneDepth) / float(BoneCount)) * Scale);
#else
	if (!bPreviousTransform)
	{
		return IntToColor(BoneIndex);
	}
	else
	{
		return float3(0.5f, 0.5f, 0.5f);
	}
#endif
}

struct FPackedBone
{
	uint BoneParent : 16;
	uint BoneDepth  : 16;
};

FPackedBone GetPackedBone(FSkinningHeader SkinningHeader, uint BoneIndex)
{
	const uint BaseOffset = SkinningHeader.HierarchyBufferOffset * (uint)sizeof(uint);
	const uint BoneOffset = BoneIndex * (uint)sizeof(uint);
	return Scene.Skinning.BoneHierarchy.Load<FPackedBone>(BaseOffset + BoneOffset);
}

float4x4 MatrixFrom4x3(float4x3 M)
{
	return float4x4(
		float4(M[0], 0.0f),
		float4(M[1], 0.0f),
		float4(M[2], 0.0f),
		float4(M[3], 1.0f)
	);
}

void LineVS(
	in uint InstanceID		: SV_InstanceID,
	in uint VertexID		: SV_VertexID,
	out float4 OutPosition	: SV_POSITION,
	out float4 OutColor		: TEXCOORD0
)
{
	ResolvedView = ResolveView();

	uint BoneIndex = VertexID >> 1u;

	const FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PersistentPrimitiveIndex);
	const FInstanceSceneData InstanceData = GetInstanceSceneData(PrimitiveData.InstanceSceneDataOffset + InstanceID);
	const FSkinningHeader SkinningHeader = LoadSkinningHeader(PersistentPrimitiveIndex);
	const FPackedBone PackedBone = GetPackedBone(SkinningHeader, BoneIndex);

	uint BoneDepth = PackedBone.BoneDepth;
	if ((VertexID % 2u) == 0)
	{
		const uint ParentBoneIndex = PackedBone.BoneParent;
		if (ParentBoneIndex < SkinningHeader.MaxTransformCount)
		{
			BoneIndex = ParentBoneIndex;
			BoneDepth--;
		}
	}

	const float4x4 BoneTransform = MatrixFrom4x3(LoadSkinningBoneTransform(GetSkinningBoneTransformOffset(SkinningHeader, InstanceData.SkinningData, bPreviousTransform) + BoneIndex));

	float4x4 BoneObjectSpace;

	BRANCH
	if (SkinningHeader.bHasScale)
	{
		BoneObjectSpace = MatrixFromBoneTransformWithScale(LoadSkinningBoneObjectSpaceWithScale(SkinningHeader.ObjectSpaceBufferOffset, BoneIndex));
	}
	else
	{
		BoneObjectSpace = MatrixFromBoneTransform(LoadSkinningBoneObjectSpace(SkinningHeader.ObjectSpaceBufferOffset, BoneIndex));
	}

	const float3 LocalPosition = mul(BoneObjectSpace, BoneTransform)._m30_m31_m32;
	const float4 TranslatedWorldPosition = float4(DFTransformLocalToTranslatedWorld(LocalPosition, InstanceData.LocalToWorld, ResolvedView.PreViewTranslation).xyz, 1.0f);

	OutPosition = mul(TranslatedWorldPosition, ResolvedView.TranslatedWorldToClip);
	OutColor = float4(GetBoneColor(VertexID >> 1u, BoneDepth, SkinningHeader.MaxTransformCount), 1.0f);
}

void LinePS(
	 in float4 Position	: SV_POSITION,
	 in float4 Color	: TEXCOORD0,
	out float4 OutColor	: SV_Target0
)
{
	OutColor = Color;
}