// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "/Engine/Shared/SkinningDefinitions.h"
#include "/Engine/Shared/HLSLStaticAssert.h"

#if USE_COMPRESSED_BONE_TRANSFORM

float4x3 UnpackCompressedBoneTransform(uint4 Data0, uint4 Data1)
{
	float4x3 Result;
	Result[0] = float3(f16tof32(Data0.w),       f16tof32(Data0.w >> 16), f16tof32(Data1.x));
	Result[1] = float3(f16tof32(Data1.x >> 16), f16tof32(Data1.y),       f16tof32(Data1.y >> 16));
	Result[2] = float3(f16tof32(Data1.z),       f16tof32(Data1.z >> 16), f16tof32(Data1.w));
	Result[3] = asfloat(Data0.xyz);
	return Result;
}

float4x3 LoadCompressedBoneTransform(FCompressedBoneTransformBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	const uint Offset = BaseOffset + BoneIndex * 32u;

	const uint4 Data0 = SrcBuffer.Load4(Offset);
	const uint4 Data1 = SrcBuffer.Load4(Offset + 16);

	return UnpackCompressedBoneTransform(Data0, Data1);
}

float3x4 UnpackCompressedBoneTransformTransposed(uint4 Data0, uint4 Data1)
{
	float3x4 Result;
	Result[0] = float4(f16tof32(Data0.w),       f16tof32(Data1.x >> 16), f16tof32(Data1.z),       asfloat(Data0.x));
	Result[1] = float4(f16tof32(Data0.w >> 16), f16tof32(Data1.y),       f16tof32(Data1.z >> 16), asfloat(Data0.y));
	Result[2] = float4(f16tof32(Data1.x),       f16tof32(Data1.y >> 16), f16tof32(Data1.w),       asfloat(Data0.z));
	return Result;
}

float3x4 LoadCompressedBoneTransformTransposed(FCompressedBoneTransformBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	const uint Offset = BaseOffset + BoneIndex * 32u;

	const uint4 Data0 = SrcBuffer.Load4(Offset);
	const uint4 Data1 = SrcBuffer.Load4(Offset + 16);

	return UnpackCompressedBoneTransformTransposed(Data0, Data1);
}

float4x3 LoadCompressedBoneTransform(FCompressedBoneTransformRWBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	const uint Offset = BaseOffset + BoneIndex * 32u;

	const uint4 Data0 = SrcBuffer.Load4(Offset);
	const uint4 Data1 = SrcBuffer.Load4(Offset + 16);

	return UnpackCompressedBoneTransform(Data0, Data1);
}

void StoreCompressedBoneTransform(FCompressedBoneTransformRWBuffer DstBuffer, uint BaseOffset, uint BoneIndex, float4x3 BoneTransform)
{
	const uint Offset = BaseOffset + BoneIndex * 32u;

	const uint3 XAxis = f32tof16(BoneTransform[0]);
	const uint3 YAxis = f32tof16(BoneTransform[1]);
	const uint3 ZAxis = f32tof16(BoneTransform[2]);

	const uint4 Data0 = uint4(asuint(BoneTransform[3]), XAxis.x | (XAxis.y << 16));
	const uint4 Data1 = uint4(XAxis.z | (YAxis.x << 16), YAxis.y | (YAxis.z << 16), ZAxis.x | (ZAxis.y << 16), ZAxis.z);
	
	DstBuffer.Store4(Offset,		Data0);
	DstBuffer.Store4(Offset + 16,	Data1);
}

#else
//;

float3x4 LoadCompressedBoneTransformTransposed(FCompressedBoneTransformBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	const uint BaseIndex = (BaseOffset >> 4) + BoneIndex * 3;

	return float3x4(SrcBuffer[BaseIndex], SrcBuffer[BaseIndex + 1], SrcBuffer[BaseIndex + 2]);
}

float4x3 LoadCompressedBoneTransform(FCompressedBoneTransformBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	return transpose(LoadCompressedBoneTransformTransposed(SrcBuffer, BaseOffset, BoneIndex));
}

float4x3 LoadCompressedBoneTransform(FCompressedBoneTransformRWBuffer SrcBuffer, uint BaseOffset, uint BoneIndex)
{
	const uint BaseIndex = (BaseOffset >> 4) + BoneIndex * 3;

	return transpose(float3x4(SrcBuffer[BaseIndex], SrcBuffer[BaseIndex + 1], SrcBuffer[BaseIndex + 2]));
}

#if COMPILER_SUPPORTS_HLSL2021

void StoreCompressedBoneTransform(FCompressedBoneTransformRWBuffer DstBuffer, uint BaseOffset, uint BoneIndex, float4x3 BoneTransform)
{
	const uint BaseIndex = (BaseOffset >> 4) + BoneIndex * 3;
	float3x4 Transposed = transpose(BoneTransform);
	DstBuffer[BaseIndex + 0] = Transposed[0];
	DstBuffer[BaseIndex + 1] = Transposed[1];
	DstBuffer[BaseIndex + 2] = Transposed[2];
}

#endif

#endif // !USE_COMPRESSED_BONE_TRANSFORM