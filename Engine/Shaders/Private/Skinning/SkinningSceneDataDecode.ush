// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkinningDataDecode.ush"
#include "../SceneData.ush"
#include "../BitPacking.ush"
#include "../BoneTransform.ush"

float4x3 LoadSkinningBoneTransform(uint TransformIndex)
{
	return LoadCompressedBoneTransform(Scene.Skinning.BoneTransforms, 0, TransformIndex);
}

float3x4 LoadSkinningBoneTransformTransposed(uint TransformIndex)
{
	return LoadCompressedBoneTransformTransposed(Scene.Skinning.BoneTransforms, 0,TransformIndex);
}

#if COMPILER_SUPPORTS_HLSL2021

FSkinningHeader LoadSkinningHeader(uint InPrimitiveIndex)
{
	const uint Offset = InPrimitiveIndex * (uint)sizeof(FSkinningHeader);
	return Scene.Skinning.Headers.Load<FSkinningHeader>(Offset);
}

FBoneTransformWithScale LoadSkinningBoneObjectSpaceWithScale(uint BufferOffset, uint BoneIndex)
{
	const uint BufferOffsetBytes = BufferOffset * (uint)sizeof(float);
	return Scene.Skinning.BoneObjectSpace.Load<FBoneTransformWithScale>(BufferOffsetBytes + BoneIndex * (uint)sizeof(FBoneTransformWithScale));
}

FBoneTransform LoadSkinningBoneObjectSpace(uint BufferOffset, uint BoneIndex)
{
	const uint BufferOffsetBytes = BufferOffset * (uint)sizeof(float);
	return Scene.Skinning.BoneObjectSpace.Load<FBoneTransform>(BufferOffsetBytes + BoneIndex * (uint)sizeof(FBoneTransform));
}

uint GetSkinningBoneTransformOffset(FSkinningHeader SkinningHeader, uint InstanceIndex, bool bPrevious)
{
	return SkinningHeader.TransformBufferOffset + (InstanceIndex * (SkinningHeader.bIsRefPose ? 0 : 1) + (bPrevious ^ SkinningHeader.CurrentTransformSlot)) * SkinningHeader.MaxTransformCount;
}

uint GetSkinningBoneTransformOffset(uint PrimitiveIndex, uint InstanceIndex, bool bPrevious)
{
	FSkinningHeader SkinningHeader = LoadSkinningHeader(PrimitiveIndex);
	return GetSkinningBoneTransformOffset(SkinningHeader, InstanceIndex, bPrevious);
}

#endif // !COMPILER_SUPPORTS_HLSL2021