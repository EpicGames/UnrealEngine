// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/SceneData.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "SkinningDataDecode.ush"

struct FTransformBlockHeader
{
	uint BlockLocalIndex;
	uint BlockTransformCount;
	uint BlockTransformOffset;
};

FCompressedBoneTransformRWBuffer TransformBuffer;
StructuredBuffer<FTransformBlockHeader> HeaderBuffer;

[numthreads(TRANSFORMS_PER_GROUP, 1, 1)]
void RefPoseProviderCS(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint GlobalBlockIndex		= GroupId.x;
	const uint LocalTransformIndex	= GroupIndex;

	const FTransformBlockHeader BlockHeader = HeaderBuffer[GlobalBlockIndex];
	if (LocalTransformIndex < BlockHeader.BlockTransformCount)
	{
		const float4x3 Identity =
		{
			float3(1.0f, 0.0f, 0.0f),
			float3(0.0f, 1.0f, 0.0f),
			float3(0.0f, 0.0f, 1.0f),
			float3(0.0f, 0.0f, 0.0f)
		};

		// Every primitive has 2 transforms (current 3x4 and previous 3x4 packed together)
		StoreCompressedBoneTransform(TransformBuffer, BlockHeader.BlockTransformOffset, LocalTransformIndex * 2, Identity);
		StoreCompressedBoneTransform(TransformBuffer, BlockHeader.BlockTransformOffset, LocalTransformIndex * 2 + 1, Identity);
	}
}