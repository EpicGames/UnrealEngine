// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphere.usf: Sky and atmosphere rendering functions.  
=============================================================================*/


// Change this to force recompilation of all volumetric cloud material shaders
#pragma message("UESHADERMETADATA_VERSION 849A6B91-7442-4D77-AA37-77C896193CD5")


#include "Common.ush"
#include "SceneTexturesCommon.ush"
#include "/Engine/Shared/EnvironmentComponentsFlags.h"

#ifndef COLORED_TRANSMITTANCE_ENABLED	// Never used, UE4 does not supports dual blending
#define COLORED_TRANSMITTANCE_ENABLED 0 
#endif
#ifndef MULTISCATTERING_APPROX_SAMPLING_ENABLED
#define MULTISCATTERING_APPROX_SAMPLING_ENABLED 0
#endif
#ifndef HIGHQUALITY_MULTISCATTERING_APPROX_ENABLED
#define HIGHQUALITY_MULTISCATTERING_APPROX_ENABLED 0
#endif
#ifndef FASTSKY_ENABLED 
#define FASTSKY_ENABLED 0 
#endif
#ifndef FASTAERIALPERSPECTIVE_ENABLED 
#define FASTAERIALPERSPECTIVE_ENABLED 0 
#endif
#ifndef SOURCE_DISK_ENABLED
#define SOURCE_DISK_ENABLED 0
#endif
#ifndef PER_PIXEL_NOISE
#define PER_PIXEL_NOISE 0
#endif
#ifndef SECOND_ATMOSPHERE_LIGHT_ENABLED
#define SECOND_ATMOSPHERE_LIGHT_ENABLED 0
#endif
#ifndef RENDERSKY_ENABLED
#define RENDERSKY_ENABLED 0
#endif

#ifndef TRANSMITTANCE_PASS
#define TRANSMITTANCE_PASS 0
#endif
#ifndef MULTISCATT_PASS
#define MULTISCATT_PASS 0
#endif
#ifndef SKYLIGHT_PASS
#define SKYLIGHT_PASS 0
#endif
#ifndef SKYVIEWLUT_PASS
#define SKYVIEWLUT_PASS 0
#endif

#ifndef SAMPLE_OPAQUE_SHADOW
#define SAMPLE_OPAQUE_SHADOW 0
#endif
#ifndef SAMPLE_CLOUD_SHADOW
#define SAMPLE_CLOUD_SHADOW 0
#endif
#ifndef SAMPLE_CLOUD_SKYAO
#define SAMPLE_CLOUD_SKYAO 0
#endif
#ifndef SAMPLE_ATMOSPHERE_ON_CLOUDS
#define SAMPLE_ATMOSPHERE_ON_CLOUDS 0
#endif

#ifndef VIRTUAL_SHADOW_MAP
#define VIRTUAL_SHADOW_MAP 0
#endif

#ifndef SEPARATE_MIE_RAYLEIGH_SCATTERING
#define SEPARATE_MIE_RAYLEIGH_SCATTERING 0
#endif

#if SAMPLE_OPAQUE_SHADOW

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#include "ShadowProjectionCommon.ush"
#include "ShadowFilteringCommon.ush"

#define VOLUME_SHADOW_SAMPLING_INPUT 0
#include "VolumeLightingCommonSampling.ush"
#undef VOLUME_SHADOW_SAMPLING_INPUT

#if SECOND_ATMOSPHERE_LIGHT_ENABLED
#define VOLUME_SHADOW_SAMPLING_INPUT 1
#include "VolumeLightingCommonSampling.ush"
#undef VOLUME_SHADOW_SAMPLING_INPUT
#endif

#if VIRTUAL_SHADOW_MAP
#include "VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#endif // SAMPLE_OPAQUE_SHADOW

#define BlueNoise SkyAtmosphere.BlueNoise
#include "BlueNoise.ush"

#if (SAMPLE_CLOUD_SHADOW || SAMPLE_CLOUD_SKYAO)
#include "VolumetricCloudCommon.ush" 
#endif

#include "SkyAtmosphereCommon.ush"

#if SAMPLE_ATMOSPHERE_ON_CLOUDS
Texture2D<float>	VolumetricCloudDepthTexture;
Texture2D<float4>	InputCloudLuminanceTransmittanceTexture;
#endif

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, MSAA_SAMPLE_COUNT> MSAADepthTexture;
#endif


// View data is not available for passes running once per scene (and not once per view).
#define VIEWDATA_AVAILABLE (TRANSMITTANCE_PASS!=1 && MULTISCATT_PASS!=1 && SKYLIGHT_PASS!=1)

 // FASTSKY mapping is done based on Light0 as main light
#define FASTSKY_LIGHT_INDEX 0

// Only available for shaders ran per view, so this constant should not be accessed in common code such as IntegrateSingleScatteredLuminance for instance.
float AerialPerspectiveStartDepthKm;

// Propagate alpha with (View.RenderingReflectionCaptureMask == 0.0f) guarantee
uint bPropagateAlphaNonReflection;

// - RayOrigin: ray origin
// - RayDir: normalized ray direction
// - SphereCenter: sphere center
// - SphereRadius: sphere radius
// - Returns distance from RayOrigin to closest intersecion with sphere,
//   or -1.0 if no intersection.
float RaySphereIntersectNearest(float3 RayOrigin, float3 RayDir, float3 SphereCenter, float SphereRadius)
{
	float2 Sol = RayIntersectSphere(RayOrigin, RayDir, float4(SphereCenter, SphereRadius));
	float Sol0 = Sol.x;
	float Sol1 = Sol.y;
	if (Sol0 < 0.0f && Sol1 < 0.0f)
	{
		return -1.0f;
	}
	if (Sol0 < 0.0f)
	{
		return max(0.0f, Sol1);
	}
	else if (Sol1 < 0.0f)
	{
		return max(0.0f, Sol0);
	}
	return max(0.0f, min(Sol0, Sol1));
}


float4 GetScreenTranslatedWorldPos(float4 SVPos, float DeviceZ)
{
	DeviceZ = max(0.000000000001, DeviceZ);	// TODO: investigate why SvPositionToWorld returns bad values when DeviceZ is far=0 when using inverted z
	return float4(SvPositionToTranslatedWorld(float4(SVPos.xy, DeviceZ, 1.0)), 1.0);
}

bool MoveToTopAtmosphere(inout float3 WorldPos, in float3 WorldDir, in float AtmosphereTopRadius)
{
	float ViewHeight = length(WorldPos);
	if (ViewHeight > AtmosphereTopRadius)
	{
		float TTop = RaySphereIntersectNearest(WorldPos, WorldDir, float3(0.0f, 0.0f, 0.0f), AtmosphereTopRadius);
		if (TTop >= 0.0f)
		{
			float3 UpVector = WorldPos / ViewHeight;
			float3 UpOffset = UpVector * -PLANET_RADIUS_OFFSET;
			WorldPos = WorldPos + WorldDir * TTop + UpOffset;
		}
		else
		{
			// Ray is not intersecting the atmosphere
			return false;
		}
	}
	return true; // ok to start tracing
}

////////////////////////////////////////////////////////////
// LUT functions
////////////////////////////////////////////////////////////

// Transmittance LUT function parameterisation from Bruneton 2017 https://github.com/ebruneton/precomputed_atmospheric_scattering
// uv in [0,1]
// ViewZenithCosAngle in [-1,1]
// ViewHeight in [bottomRAdius, topRadius]

void UvToLutTransmittanceParams(out float ViewHeight, out float ViewZenithCosAngle, in float2 UV)
{
	//UV = FromSubUvsToUnit(UV, SkyAtmosphere.TransmittanceLutSizeAndInvSize); // No real impact so off
	fromTransmittanceLutUVs(ViewHeight, ViewZenithCosAngle, Atmosphere.BottomRadiusKm, Atmosphere.TopRadiusKm, UV);
}

void LutTransmittanceParamsToUv(in float ViewHeight, in float ViewZenithCosAngle, out float2 UV)
{
	getTransmittanceLutUvs(ViewHeight, ViewZenithCosAngle, Atmosphere.BottomRadiusKm, Atmosphere.TopRadiusKm, UV);
}

// SkyViewLut is a new texture used for fast sky rendering.
// It is low resolution of the sky rendering around the camera,
// basically a lat/long parameterisation with more texel close to the horizon for more accuracy during sun set.

void UvToSkyViewLutParams(out float3 ViewDir, in float ViewHeight, in float2 UV)
{
	// Constrain uvs to valid sub texel range (avoid zenith derivative issue making LUT usage visible)
	UV = FromSubUvsToUnit(UV, SkyAtmosphere.SkyViewLutSizeAndInvSize);

	float Vhorizon = sqrt(ViewHeight * ViewHeight - Atmosphere.BottomRadiusKm * Atmosphere.BottomRadiusKm);
	float CosBeta = Vhorizon / ViewHeight;				// cos of zenith angle from horizon to zeniht
	float Beta = acosFast4(CosBeta);
	float ZenithHorizonAngle = PI - Beta;

	float ViewZenithAngle;
	if (UV.y < 0.5f)
	{
		float Coord = 2.0f * UV.y;
		Coord = 1.0f - Coord;
		Coord *= Coord;
		Coord = 1.0f - Coord;
		ViewZenithAngle = ZenithHorizonAngle * Coord;
	}
	else
	{
		float Coord = UV.y * 2.0f - 1.0f;
		Coord *= Coord;
		ViewZenithAngle = ZenithHorizonAngle + Beta * Coord;
	}
	float CosViewZenithAngle = cos(ViewZenithAngle);
	float SinViewZenithAngle = sqrt(1.0 - CosViewZenithAngle * CosViewZenithAngle) * (ViewZenithAngle > 0.0f ? 1.0f : -1.0f); // Equivalent to sin(ViewZenithAngle)

	float LongitudeViewCosAngle = UV.x * 2.0f * PI;

	// Make sure those values are in range as it could disrupt other math done later such as sqrt(1.0-c*c)
	float CosLongitudeViewCosAngle = cos(LongitudeViewCosAngle);
	float SinLongitudeViewCosAngle = sqrt(1.0 - CosLongitudeViewCosAngle * CosLongitudeViewCosAngle) * (LongitudeViewCosAngle <= PI ? 1.0f : -1.0f); // Equivalent to sin(LongitudeViewCosAngle)
	ViewDir = float3(
		SinViewZenithAngle * CosLongitudeViewCosAngle,
		SinViewZenithAngle * SinLongitudeViewCosAngle,
		CosViewZenithAngle
		);
}


////////////////////////////////////////////////////////////
// Utilities
////////////////////////////////////////////////////////////

#if VIEWDATA_AVAILABLE
// Exposure used for regular views by the FastSky and AP LUTs.
#define ViewPreExposure			View.PreExposure
#define ViewOneOverPreExposure	View.OneOverPreExposure
// When rendering a real time reflection capture (sky envmap) whe use a different output exposure
#define OutputPreExposure		(View.RealTimeReflectionCapture ? View.RealTimeReflectionCapturePreExposure : View.PreExposure)
#else
#define ViewPreExposure			1.0f
#define ViewOneOverPreExposure	1.0f
#define OutputPreExposure		1.0f
#endif


////////////////////////////////////////////////////////////
// Real time reflection capture overriden parameters
////////////////////////////////////////////////////////////

#if PERMUTATION_REALTIME_REFLECTION_LUT

#undef OutputPreExposure
#define OutputPreExposure View.PreExposure

float3x3 GetSkyViewLutReferentialParameter()
{
	return GetSkyViewLutReferential(SkyAtmosphereRealTimeReflectionLUTParameters.SkyViewLutReferential);
}

float3 GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter()
{
	return SkyAtmosphereRealTimeReflectionLUTParameters.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz;
}

float3 GetSkyCameraTranslatedWorldOriginParameter()
{
	return SkyAtmosphereRealTimeReflectionLUTParameters.SkyCameraTranslatedWorldOrigin;
}

#else // PERMUTATION_REALTIME_REFLECTION_LUT

float3x3 GetSkyViewLutReferentialParameter()
{
	return GetSkyViewLutReferential(View.SkyViewLutReferential);
}

float3 GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter()
{
	return View.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz;
}

float3 GetSkyCameraTranslatedWorldOriginParameter()
{
	return View.SkyCameraTranslatedWorldOrigin;
}

#endif // PERMUTATION_REALTIME_REFLECTION_LUT

// This is the world position of the camera. It is also force to be at the top of the virutal planet surface. 
// This is to always see the sky even when the camera is buried into the virtual planet.
float3 GetCameraTranslatedWorldPos()
{
	return GetSkyCameraTranslatedWorldOriginParameter();
}

// This is the camera position relative to the virtual planet center.
// This is convenient because for all the math in this file using world position relative to the virtual planet center.
float3 GetTranslatedCameraPlanetPos()
{
	return (GetCameraTranslatedWorldPos() - GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter()) * CM_TO_SKY_UNIT;
}



SamplerState TransmittanceLutTextureSampler;
SamplerState MultiScatteredLuminanceLutTextureSampler;
SamplerState SkyViewLutTextureSampler;
SamplerState CameraAerialPerspectiveVolumeTextureSampler;
SamplerState VolumetricCloudShadowMapTexture0Sampler;
SamplerState VolumetricCloudShadowMapTexture1Sampler;
SamplerState VolumetricCloudSkyAOTextureSampler;

Texture2D<float4> TransmittanceLutTexture;
Texture2D<float4> MultiScatteredLuminanceLutTexture;
Texture2D<float4> SkyViewLutTexture;
Texture3D<float4> CameraAerialPerspectiveVolumeTexture;
Texture2D<float3> VolumetricCloudShadowMapTexture0;
Texture2D<float3> VolumetricCloudShadowMapTexture1;
Texture2D<float3> VolumetricCloudSkyAOTexture;
float VolumetricCloudShadowStrength0;
float VolumetricCloudShadowStrength1;

int VirtualShadowMapId0;
int VirtualShadowMapId1;


#if SOURCE_DISK_ENABLED

uint SourceDiskEnabled; 

float3 GetLightDiskLuminance(float3 PlanetCenterToCamera, float3 WorldDir, uint LightIndex)
{
	float t = RaySphereIntersectNearest(PlanetCenterToCamera, WorldDir, float3(0.0f, 0.0f, 0.0f), Atmosphere.BottomRadiusKm);
	if (t < 0.0f												// No intersection with the planet
		&& View.RenderingReflectionCaptureMask==0.0f)	// Do not render light disk when in reflection capture in order to avoid double specular. The sun contribution is already computed analyticaly.
	{
		// GetLightDiskLuminance contains a tiny soft edge effect
		float3 LightDiskLuminance = GetLightDiskLuminance(
			PlanetCenterToCamera, WorldDir, Atmosphere.BottomRadiusKm, Atmosphere.TopRadiusKm,
			TransmittanceLutTexture, TransmittanceLutTextureSampler,
			View.AtmosphereLightDirection[LightIndex].xyz, View.AtmosphereLightDiscCosHalfApexAngle_PPTrans[LightIndex].x, View.AtmosphereLightDiscLuminance[LightIndex].xyz);

		// Clamp to avoid crazy high values (and exposed 64000.0f luminance is already crazy high, solar system sun is 1.6x10^9). Also this removes +inf float and helps TAA.
		const float3 MaxLightLuminance = 64000.0f;
		float3 ExposedLightLuminance = LightDiskLuminance * OutputPreExposure;
		ExposedLightLuminance = min(ExposedLightLuminance, MaxLightLuminance);

		return ExposedLightLuminance;
	}
	return 0.0f;
}
#endif


float3 GetMultipleScattering(float3 WorlPos, float ViewZenithCosAngle)
{
	float2 UV = saturate(float2(ViewZenithCosAngle*0.5f + 0.5f, (length(WorlPos) - Atmosphere.BottomRadiusKm) / (Atmosphere.TopRadiusKm - Atmosphere.BottomRadiusKm)));
	// We do no apply UV transform to sub range here as it has minimal impact.
	float3 MultiScatteredLuminance = MultiScatteredLuminanceLutTexture.SampleLevel(MultiScatteredLuminanceLutTextureSampler, UV, 0).rgb;
	return MultiScatteredLuminance;
}


float3 GetTransmittance(in float LightZenithCosAngle, in float PHeight)
{
	float2 UV;
	LutTransmittanceParamsToUv(PHeight, LightZenithCosAngle, UV);
#ifdef WHITE_TRANSMITTANCE
	float3 TransmittanceToLight = 1.0f;
#else
	float3 TransmittanceToLight = TransmittanceLutTexture.SampleLevel(TransmittanceLutTextureSampler, UV, 0).rgb;
	TransmittanceToLight = DecodeTransmittance(TransmittanceToLight);
#endif
	return TransmittanceToLight;
}


#define DEFAULT_SAMPLE_OFFSET 0.3f
float SkyAtmosphereNoise(float2 UV)
{
	//	return DEFAULT_SAMPLE_OFFSET;
	//	return float(Rand3DPCG32(int3(UV.x, UV.y, S)).x) / 4294967296.0f;
#if VIEWDATA_AVAILABLE && PER_PIXEL_NOISE
#if 0 // Simple noise
	return View.RealTimeReflectionCapture ? DEFAULT_SAMPLE_OFFSET : InterleavedGradientNoise(UV.xy, float(View.StateFrameIndexMod8));
#else // Blue noise works best
	return View.RealTimeReflectionCapture ? DEFAULT_SAMPLE_OFFSET : BlueNoiseScalar(uint2(UV), View.StateFrameIndex);
#endif
#else
	return DEFAULT_SAMPLE_OFFSET;
#endif
}



////////////////////////////////////////////////////////////
// Main scattering/transmitance integration function
////////////////////////////////////////////////////////////

struct SingleScatteringResult
{
	float3 L;						// Scattered light (luminance)
	float3 LMieOnly;				// L but Mie scattering only
	float3 LRayOnly;				// L but Rayleigh scattering only
	float3 OpticalDepth;			// Optical depth (1/m)
	float3 Transmittance;			// Transmittance in [0,1] (unitless)
	float3 TransmittanceMieOnly;	// Transmittance in [0,1] (unitless) but Mie scattering only
	float3 TransmittanceRayOnly;	// Transmittance in [0,1] (unitless) but Rayleigh scattering only
	float3 MultiScatAs1;
};

struct SamplingSetup
{
	bool VariableSampleCount;
	float SampleCountIni;			// Used when VariableSampleCount is false
	float MinSampleCount;
	float MaxSampleCount;
	float DistanceToSampleCountMaxInv;
};

// In this function, all world position are relative to the planet center (itself expressed within translated world space)
SingleScatteringResult IntegrateSingleScatteredLuminance(
	in float4 SVPos, in float3 WorldPos, in float3 WorldDir,
	in bool Ground, in SamplingSetup Sampling, in float DeviceZ, in bool MieRayPhase,
	in float3 Light0Dir, in float3 Light1Dir, in float3 Light0Illuminance, in float3 Light1Illuminance,
	in float AerialPespectiveViewDistanceScale,
	in float tMaxMax = 9000000.0f)
{
	SingleScatteringResult Result;
	Result.L = 0;
	Result.LMieOnly = 0;
	Result.LRayOnly = 0;
	Result.OpticalDepth = 0;
	Result.Transmittance = 1.0f;
	Result.TransmittanceMieOnly = 1.0f;
	Result.TransmittanceRayOnly = 1.0f;
	Result.MultiScatAs1 = 0;

	if (dot(WorldPos, WorldPos) <= Atmosphere.BottomRadiusKm * Atmosphere.BottomRadiusKm)
	{
		return Result;	// Camera is inside the planet ground
	}

	float2 PixPos = SVPos.xy;

	// Compute next intersection with atmosphere or ground
	float3 PlanetO = float3(0.0f, 0.0f, 0.0f);
	float tMax = 0.0f;
#if 0
	// The bottom code causes the skyview lut to flicker when view from space afar.
	// Remove that code when the else section is proven.
	float tBottom = RaySphereIntersectNearest(WorldPos, WorldDir, PlanetO, Atmosphere.BottomRadiusKm);
	float tTop = RaySphereIntersectNearest(WorldPos, WorldDir, PlanetO, Atmosphere.TopRadiusKm);
	if (tBottom < 0.0f)
	{
		if (tTop < 0.0f)
		{
			tMax = 0.0f; // No intersection with planet nor its atmosphere: stop right away  
			return Result;
		}
		else
		{
			tMax = tTop;
		}
	}
	else
	{
		if (tTop > 0.0f)
		{
			tMax = min(tTop, tBottom);
		}
	}
#else
	float tBottom = 0.0f;
	float2 SolB = RayIntersectSphere(WorldPos, WorldDir, float4(PlanetO, Atmosphere.BottomRadiusKm));
	float2 SolT = RayIntersectSphere(WorldPos, WorldDir, float4(PlanetO, Atmosphere.TopRadiusKm));

	const bool bNoBotIntersection = all(SolB < 0.0f);
	const bool bNoTopIntersection = all(SolT < 0.0f);
	if (bNoTopIntersection)
	{
		// No intersection with planet or its atmosphere.
		tMax = 0.0f;
		return Result;
	}
	else if (bNoBotIntersection)
	{
		// No intersection with planet, so we trace up to the far end of the top atmosphere 
		// (looking up from ground or edges when see from afar in space).
		tMax = max(SolT.x, SolT.y);
	}
	else
	{
		// Interesection with planet and atmospehre: we simply trace up to the planet ground.
		// We know there is at least one intersection thanks to bNoBotIntersection.
		// If one of the solution is invalid=-1, that means we are inside the planet: we stop tracing by setting tBottom=0.
		tBottom = max(0.0f, min(SolB.x, SolB.y));
		tMax = tBottom;
	}
#endif

	float PlanetOnOpaque = 1.0f;	// This is used to hide opaque meshes under the planet ground
#if VIEWDATA_AVAILABLE
#if SAMPLE_ATMOSPHERE_ON_CLOUDS
	if (true)
	{
		float tDepth = DeviceZ; // When SAMPLE_ATMOSPHERE_ON_CLOUDS, DeviceZ is world distance in kilometer.
		if (tDepth < tMax)
		{
			tMax = tDepth;
		}
	}
#else // SAMPLE_ATMOSPHERE_ON_CLOUDS
	if (DeviceZ != FarDepthValue)
	{
		const float3 DepthBufferTranslatedWorldPosKm = GetScreenTranslatedWorldPos(SVPos, DeviceZ).xyz * CM_TO_SKY_UNIT;
		const float3 TraceStartTranslatedWorldPosKm  = WorldPos + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter() * CM_TO_SKY_UNIT; // apply planet offset to go back to world from planet local referencial.
		const float3 TraceStartToSurfaceWorldKm = DepthBufferTranslatedWorldPosKm - TraceStartTranslatedWorldPosKm;
		float tDepth = length(TraceStartToSurfaceWorldKm);
		if (tDepth < tMax)
		{
			tMax = tDepth;
		}
		else
		{
			// Artists did not like that we handle automatic hiding of opaque element behind the planet.
			// Now, pixel under the surface of earht will receive aerial perspective as if they were  on the ground.
			//PlanetOnOpaque = 0.0;
		}

		//if the ray intersects with the atmosphere boundary, make sure we do not apply atmosphere on surfaces are front of it. 
		if (dot(WorldDir, TraceStartToSurfaceWorldKm) < 0.0)
		{
			return Result;
		}
	}
#endif // SAMPLE_ATMOSPHERE_ON_CLOUDS
#endif
	tMax = min(tMax, tMaxMax);

	// Sample count 
	float SampleCount = Sampling.SampleCountIni;
	float SampleCountFloor = Sampling.SampleCountIni;
	float tMaxFloor = tMax;
	if (Sampling.VariableSampleCount)
	{
		SampleCount = lerp(Sampling.MinSampleCount, Sampling.MaxSampleCount, saturate(tMax*Sampling.DistanceToSampleCountMaxInv));
		SampleCountFloor = floor(SampleCount);
		tMaxFloor = tMax * SampleCountFloor / SampleCount;	// rescale tMax to map to the last entire step segment.
	}
	float dt = tMax / SampleCount;

	// Phase functions
	const float uniformPhase = 1.0f / (4.0f * PI);
	const float3 wi = Light0Dir;
	const float3 wo = WorldDir;
	float cosTheta = dot(wi, wo);
	float MiePhaseValueLight0 = HenyeyGreensteinPhase(Atmosphere.MiePhaseG, -cosTheta);	// negate cosTheta because due to WorldDir being a "in" direction. 
	float RayleighPhaseValueLight0 = RayleighPhase(cosTheta);
#if SECOND_ATMOSPHERE_LIGHT_ENABLED
	cosTheta = dot(Light1Dir, wo);
	float MiePhaseValueLight1 = HenyeyGreensteinPhase(Atmosphere.MiePhaseG, -cosTheta);	// negate cosTheta because due to WorldDir being a "in" direction. 
	float RayleighPhaseValueLight1 = RayleighPhase(cosTheta);
#endif

	// Ray march the atmosphere to integrate optical depth
	float3 L = 0.0f;
	float3 LMieOnly = 0.0f;
	float3 LRayOnly = 0.0f;
	float3 Throughput = 1.0f;
	float3 ThroughputMieOnly = 1.0f;
	float3 ThroughputRayOnly = 1.0f;
	float3 OpticalDepth = 0.0f;
	float t = 0.0f;
	float tPrev = 0.0f;

	float3 ExposedLight0Illuminance = Light0Illuminance * OutputPreExposure;
#if SECOND_ATMOSPHERE_LIGHT_ENABLED
	float3 ExposedLight1Illuminance = Light1Illuminance * OutputPreExposure;
#endif

//#if SAMPLE_OPAQUE_SHADOW
	// Get the referencial when rendering the SkyView lut being in a special Z-top space
#if SKYVIEWLUT_PASS
	float3x3 LocalReferencial = GetSkyViewLutReferentialParameter();
#endif
//#endif

	float PixelNoise = PER_PIXEL_NOISE ? SkyAtmosphereNoise(PixPos.xy) : DEFAULT_SAMPLE_OFFSET;
	for (float SampleI = 0.0f; SampleI < SampleCount; SampleI += 1.0f)
	{
		// Compute current ray t and sample point P
		if (Sampling.VariableSampleCount)
		{
			// More expenssive but artefact free
			float t0 = (SampleI) / SampleCountFloor;
			float t1 = (SampleI + 1.0f) / SampleCountFloor;;
			// Non linear distribution of samples within the range.
			t0 = t0 * t0;
			t1 = t1 * t1;
			// Make t0 and t1 world space distances.
			t0 = tMaxFloor * t0;
			if (t1 > 1.0f)
			{
				t1 = tMax;
				//t1 = tMaxFloor;	// this reveal depth slices
			}
			else
			{
				t1 = tMaxFloor * t1;
			}
			t = t0 + (t1 - t0) * PixelNoise;
			dt = t1 - t0;
		}
		else
		{
			t = tMax * (SampleI + PixelNoise) / SampleCount;
		}
		float3 P = WorldPos + t * WorldDir;
		float PHeight = length(P);

		// Sample the medium
		MediumSampleRGB Medium = SampleAtmosphereMediumRGB(P);
		const float3 SampleOpticalDepth = Medium.Extinction * dt * AerialPespectiveViewDistanceScale;
		const float3 SampleTransmittance = exp(-SampleOpticalDepth);
		OpticalDepth += SampleOpticalDepth;

		// Transmittance Ray only and Mie only set half of ozone in rayleigh and half of ozone in mie parts.
		// This is not great but I do not have any better solution. 
		// Also most of the time Ozone is high in the atmosphere so it should be fine this way.
		ThroughputMieOnly *= exp(-(Medium.ExtinctionMie + Medium.ExtinctionOzo) * dt * AerialPespectiveViewDistanceScale);
		ThroughputRayOnly *= exp(-(Medium.ExtinctionRay + Medium.ExtinctionOzo) * dt * AerialPespectiveViewDistanceScale);

		// Phase and transmittance for light 0
		const float3 UpVector = P / PHeight;
		float Light0ZenithCosAngle = dot(Light0Dir, UpVector);
		float3 TransmittanceToLight0 = GetTransmittance(Light0ZenithCosAngle, PHeight);
		float3 PhaseTimesScattering0;
		float3 PhaseTimesScattering0MieOnly;
		float3 PhaseTimesScattering0RayOnly;
		if (MieRayPhase)
		{
			PhaseTimesScattering0MieOnly= Medium.ScatteringMie * MiePhaseValueLight0;
			PhaseTimesScattering0RayOnly= Medium.ScatteringRay * RayleighPhaseValueLight0;
			PhaseTimesScattering0		= PhaseTimesScattering0MieOnly + PhaseTimesScattering0RayOnly;
		}
		else
		{
			PhaseTimesScattering0MieOnly= Medium.ScatteringMie * uniformPhase;
			PhaseTimesScattering0RayOnly= Medium.ScatteringRay * uniformPhase;
			PhaseTimesScattering0		= Medium.Scattering * uniformPhase;
		}
#if SECOND_ATMOSPHERE_LIGHT_ENABLED
		// Phase and transmittance for light 1
		float Light1ZenithCosAngle = dot(Light1Dir, UpVector);
		float3 TransmittanceToLight1 = GetTransmittance(Light1ZenithCosAngle, PHeight);
		float3 PhaseTimesScattering1;
		float3 PhaseTimesScattering1MieOnly;
		float3 PhaseTimesScattering1RayOnly;
		if (MieRayPhase)
		{
			PhaseTimesScattering1MieOnly= Medium.ScatteringMie * MiePhaseValueLight1;
			PhaseTimesScattering1RayOnly= Medium.ScatteringRay * RayleighPhaseValueLight1;
			PhaseTimesScattering1		= PhaseTimesScattering1MieOnly + PhaseTimesScattering1RayOnly;
		}
		else
		{
			PhaseTimesScattering1MieOnly= Medium.ScatteringMie * uniformPhase;
			PhaseTimesScattering1RayOnly= Medium.ScatteringRay * uniformPhase;
			PhaseTimesScattering1		= Medium.Scattering * uniformPhase;
		}
#endif

		// Multiple scattering approximation
		float3 MultiScatteredLuminance0 = 0.0f;
#if MULTISCATTERING_APPROX_SAMPLING_ENABLED
		MultiScatteredLuminance0 = GetMultipleScattering(P, Light0ZenithCosAngle);
#endif
#if SECOND_ATMOSPHERE_LIGHT_ENABLED
		float3 MultiScatteredLuminance1 = 0.0f;
	#if MULTISCATTERING_APPROX_SAMPLING_ENABLED
		MultiScatteredLuminance1 = GetMultipleScattering(P, Light1ZenithCosAngle);
	#endif
#endif

		// Planet shadow
		float tPlanet0 = RaySphereIntersectNearest(P, Light0Dir, PlanetO + PLANET_RADIUS_OFFSET * UpVector, Atmosphere.BottomRadiusKm);
		float PlanetShadow0 = tPlanet0 >= 0.0f ? 0.0f : 1.0f;
		float3 ShadowP0 = P;
		bool bUnused = false;
#if SKYVIEWLUT_PASS
		ShadowP0 = GetTranslatedCameraPlanetPos() + t * mul(LocalReferencial, WorldDir); // Inverse of the local SkyViewLUT referencial transform
#endif
#if SAMPLE_OPAQUE_SHADOW
		{
			float3 ShadowSampleWorldPosition0 = ShadowP0 * SKY_UNIT_TO_CM + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter();
			PlanetShadow0 *= ComputeLight0VolumeShadowing(ShadowSampleWorldPosition0 /* - DFHackToFloat(PrimaryView.PreViewTranslation)*/, false, false, bUnused);

		#if VIRTUAL_SHADOW_MAP
			if (VirtualShadowMapId0 != INDEX_NONE)
			{
				FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapDirectional(VirtualShadowMapId0, ShadowSampleWorldPosition0);
				PlanetShadow0 *= VirtualShadowMapSample.ShadowFactor;
			}
		#endif // VIRTUALSHADOW_MAP
		}
#endif
#if SAMPLE_CLOUD_SKYAO
		float OutOpticalDepth = 0.0f;
		MultiScatteredLuminance0 *= GetCloudVolumetricShadow(ShadowP0 * SKY_UNIT_TO_CM + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter(), VolumetricCloudCommonParameters.CloudSkyAOTranslatedWorldToLightClipMatrix,
			VolumetricCloudCommonParameters.CloudSkyAOFarDepthKm, VolumetricCloudSkyAOTexture, VolumetricCloudSkyAOTextureSampler, OutOpticalDepth);
#endif
#if SAMPLE_CLOUD_SHADOW
		float OutOpticalDepth2 = 0.0f;
		PlanetShadow0 *= saturate(lerp(1.0f, GetCloudVolumetricShadow(ShadowP0 * SKY_UNIT_TO_CM + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter(), VolumetricCloudCommonParameters.CloudShadowmapTranslatedWorldToLightClipMatrix[0],
			VolumetricCloudCommonParameters.CloudShadowmapFarDepthKm[0].x, VolumetricCloudShadowMapTexture0, VolumetricCloudShadowMapTexture0Sampler, OutOpticalDepth2), VolumetricCloudShadowStrength0));
#endif
		// MultiScatteredLuminance is already pre-exposed, atmospheric light contribution needs to be pre exposed
		// Multi-scattering is also not affected by PlanetShadow or TransmittanceToLight because it contains diffuse light after single scattering.
		float3 S		= ExposedLight0Illuminance * (PlanetShadow0 * TransmittanceToLight0 * PhaseTimesScattering0			+ MultiScatteredLuminance0 * Medium.Scattering);
		float3 SMieOnly = ExposedLight0Illuminance * (PlanetShadow0 * TransmittanceToLight0 * PhaseTimesScattering0MieOnly	+ MultiScatteredLuminance0 * Medium.ScatteringMie);
		float3 SRayOnly = ExposedLight0Illuminance * (PlanetShadow0 * TransmittanceToLight0 * PhaseTimesScattering0RayOnly	+ MultiScatteredLuminance0 * Medium.ScatteringRay);

#if SECOND_ATMOSPHERE_LIGHT_ENABLED
		float tPlanet1 = RaySphereIntersectNearest(P, Light1Dir, PlanetO + PLANET_RADIUS_OFFSET * UpVector, Atmosphere.BottomRadiusKm);
		float PlanetShadow1 = tPlanet1 >= 0.0f ? 0.0f : 1.0f;
		float3 ShadowP1 = P;
#if SAMPLE_OPAQUE_SHADOW
	#if SKYVIEWLUT_PASS
		ShadowP1 = GetTranslatedCameraPlanetPos() + t * mul(LocalReferencial, WorldDir); // Inverse of the local SkyViewLUT referencial transform
	#endif
		{
			float3 ShadowSampleWorldPosition1 = ShadowP1 * SKY_UNIT_TO_CM + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter();
			PlanetShadow1 *= ComputeLight1VolumeShadowing(ShadowSampleWorldPosition1/* - DFHackToFloat(PrimaryView.PreViewTranslation)*/, false, false, bUnused);
		#if VIRTUAL_SHADOW_MAP
			if (VirtualShadowMapId1 != INDEX_NONE)
			{
				FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapDirectional(VirtualShadowMapId1, ShadowSampleWorldPosition1);
				PlanetShadow1 *= VirtualShadowMapSample.ShadowFactor;
			}
		#endif // VIRTUALSHADOW_MAP
		}
#endif // SAMPLE_OPAQUE_SHADOW
#if SAMPLE_CLOUD_SHADOW
		float OutOpticalDepth3 = 0.0f;
		PlanetShadow1 *= saturate(lerp(1.0f, GetCloudVolumetricShadow(ShadowP1 * SKY_UNIT_TO_CM + GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter(), VolumetricCloudCommonParameters.CloudShadowmapTranslatedWorldToLightClipMatrix[1],
			VolumetricCloudCommonParameters.CloudShadowmapFarDepthKm[1].x, VolumetricCloudShadowMapTexture1, VolumetricCloudShadowMapTexture1Sampler, OutOpticalDepth3), VolumetricCloudShadowStrength1));
#endif
		//  Multi-scattering can work for the second light but it is disabled for the sake of performance.
		S		 += ExposedLight1Illuminance * (PlanetShadow1 * TransmittanceToLight1 * PhaseTimesScattering1		 + MultiScatteredLuminance1 * Medium.Scattering);
		SMieOnly += ExposedLight1Illuminance * (PlanetShadow1 * TransmittanceToLight1 * PhaseTimesScattering1MieOnly + MultiScatteredLuminance1 * Medium.ScatteringMie);
		SRayOnly += ExposedLight1Illuminance * (PlanetShadow1 * TransmittanceToLight1 * PhaseTimesScattering1RayOnly + MultiScatteredLuminance1 * Medium.ScatteringRay);
#endif

		// When using the power serie to accumulate all sattering order, serie r must be <1 for a serie to converge. 
		// Under extreme coefficient, MultiScatAs1 can grow larger and thus results in broken visuals. 
		// The way to fix that is to use a proper analytical integration as porposed in slide 28 of http://www.frostbite.com/2015/08/physically-based-unified-volumetric-rendering-in-frostbite/ 
		// However, it is possible to disable as it can also work using simple power serie sum unroll up to 5th order. The rest of the orders has a really low contribution. 
#define MULTI_SCATTERING_POWER_SERIE 0 
		const float3 SafeMediumExtinction = max(Medium.Extinction, 1.e-9);
#if MULTI_SCATTERING_POWER_SERIE==0 
		// 1 is the integration of luminance over the 4pi of a sphere, and assuming an isotropic phase function of 1.0/(4*PI) 
		Result.MultiScatAs1 += Throughput * Medium.Scattering * 1.0f * dt;
#else 
		float3 MS = Medium.Scattering * 1;
		float3 MSint = (MS - MS * SampleTransmittance) / SafeMediumExtinction;
		Result.MultiScatAs1 += Throughput * MSint;
#endif 

#if 0
		L			+= Throughput * S * dt;
		LMieOnly	+= Throughput * SMieOnly * dt;
		LRayOnly	+= Throughput * SRayOnly * dt;
		Throughput	*= SampleTransmittance;
#else
		// See slide 28 at http://www.frostbite.com/2015/08/physically-based-unified-volumetric-rendering-in-frostbite/ 
		float3 Sint			= (S        - S        * SampleTransmittance) / SafeMediumExtinction;	// integrate along the current step segment 
		float3 SintMieOnly	= (SMieOnly - SMieOnly * SampleTransmittance) / SafeMediumExtinction;
		float3 SintRayOnly	= (SRayOnly - SRayOnly * SampleTransmittance) / SafeMediumExtinction;
		L			+= Throughput * Sint;														// accumulate and also take into account the transmittance from previous steps
		LMieOnly	+= Throughput * SintMieOnly;
		LRayOnly	+= Throughput * SintRayOnly;
		Throughput	*= SampleTransmittance;
#endif

		tPrev = t;
	}

	if (Ground && tMax == tBottom)
	{
		// Account for bounced light off the planet
		float3 P = WorldPos + tBottom * WorldDir;
		float PHeight = length(P);

		const float3 UpVector = P / PHeight;
		float Light0ZenithCosAngle = dot(Light0Dir, UpVector);
		float3 TransmittanceToLight0 = GetTransmittance(Light0ZenithCosAngle, PHeight);

		const float NdotL0 = saturate(dot(UpVector, Light0Dir));
		L += Light0Illuminance * TransmittanceToLight0 * Throughput * NdotL0 * Atmosphere.GroundAlbedo.rgb / PI;
#if SECOND_ATMOSPHERE_LIGHT_ENABLED
		{
			const float NdotL1 = saturate(dot(UpVector, Light1Dir));
			float Light1ZenithCosAngle = dot(UpVector, Light1Dir);
			float3 TransmittanceToLight1 = GetTransmittance(Light1ZenithCosAngle, PHeight);
			L += Light1Illuminance * TransmittanceToLight1 * Throughput * NdotL1 * Atmosphere.GroundAlbedo.rgb / PI;
		}
#endif
	}

	Result.L					= L;
	Result.LMieOnly				= LMieOnly;
	Result.LRayOnly				= LRayOnly;
	Result.OpticalDepth			= OpticalDepth;
	Result.Transmittance		= Throughput * PlanetOnOpaque;
	Result.TransmittanceMieOnly	= ThroughputMieOnly * PlanetOnOpaque;
	Result.TransmittanceRayOnly	= ThroughputRayOnly * PlanetOnOpaque;

	return Result;
}



////////////////////////////////////////////////////////////
// Main sky and atmosphere on opaque ray marching shaders
////////////////////////////////////////////////////////////

float StartDepthZ;

void SkyAtmosphereVS(
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION)
{
	float2 UV = -1.0f;
	UV = VertexId == 1 ? float2(-1.0f, 3.0f) : UV;
	UV = VertexId == 2 ? float2( 3.0f,-1.0f) : UV;

	Position = float4(UV, StartDepthZ, 1.0f);
}



float4 PrepareOutput(float3 PreExposedLuminance, float3 Transmittance = float3(1.0f, 1.0f, 1.0f))
{
	// Sky materials can result in high luminance values, e.g. the sun disk. 
	// This is so we make sure to at least stay within the boundaries of fp10 and not cause NaN on some platforms.
	// We also half that range to also make sure we have room for other additive elements such as bloom, clouds or particle visual effects.
	const float3 SafePreExposedLuminance = min(PreExposedLuminance, Max10BitsFloat.xxx * 0.5f);
	
	const float GreyScaleTransmittance = dot(Transmittance, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
	float4 LuminanceAlpha = float4(SafePreExposedLuminance, GreyScaleTransmittance);
	
	FLATTEN
	if(bPropagateAlphaNonReflection > 0)
	{
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
		LuminanceAlpha.rgb	= (IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags) && View.bPrimitiveAlphaHoldoutEnabled) ? 0.0f : LuminanceAlpha.rgb;
#endif
		LuminanceAlpha.a	= 1 - GreyScaleTransmittance;
	}
	
	return LuminanceAlpha;

}

uint DepthReadDisabled;

void UpdateVisibleSkyAlpha(in float DeviceZ, inout float4 OutLuminance)
{
	// This is for parity with the sky dome mesh behavior.
	// Output fully opaque alpha if looking through outer space, if atmosphere is not in holdout.
	bool bOutputFullyOpaque = (bPropagateAlphaNonReflection > 0) && (DeviceZ == FarDepthValue);
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	bOutputFullyOpaque &= !IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags) || !View.bPrimitiveAlphaHoldoutEnabled;
#endif

	FLATTEN
	if (bOutputFullyOpaque)
	{
		OutLuminance.a = 1.0;
	}
}

void RenderSkyAtmosphereRayMarchingPS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminance : SV_Target0
#if COLORED_TRANSMITTANCE_ENABLED
	, float4 Transmittance : SV_TARGET1;	// For dual source blending when available
#endif
#if MSAA_SAMPLE_COUNT > 1
	, in uint SampleIndex : SV_SampleIndex
#endif
)
{
	OutLuminance = 0;

	float2 PixPos = SVPos.xy;
	float2 UvBuffer = PixPos * View.BufferSizeAndInvSize.zw;	// Uv for depth buffer read (size can be larger than viewport)

	// Debug print a texture
	//const float displaySize = 256.0f;
	//if(all(PixPos<displaySize))
	//{
	//	OutLuminance = PrepareOutput(SkyViewLutTexture.SampleLevel(SkyViewLutTextureSampler, PixPos / displaySize, 0).rgb);
	//	//OutLuminance = PrepareOutput(CameraAerialPerspectiveVolumeTexture.SampleLevel(CameraAerialPerspectiveVolumeTextureSampler, float3(PixPos / displaySize, PixPos.x/ displaySize), 0).rgb * ViewOneOverPreExposure);
	//	return;
	//}

	// World position are relative to the planet center (itself expressed within translated world space)
	float3 WorldDir = GetScreenWorldDir(SVPos);
	float3 WorldPos = GetTranslatedCameraPlanetPos();
	if(IsOrthoProjection())
	{
		WorldPos += GetTranslatedWorldCameraPosFromView(SVPos.xy, true);
	}

	float3 PreExposedL = 0;
	float3 LuminanceScale = 1.0f;

#if SAMPLE_ATMOSPHERE_ON_CLOUDS
	// We could read cloud color and skip if transmittance<0.999. Could do that if it would be a compute shader.

	const float4 CloudLuminanceTransmittance = InputCloudLuminanceTransmittanceTexture.Load(int3(PixPos, 0));
	const float CloudCoverage = 1.0f - CloudLuminanceTransmittance.a;
	if (CloudLuminanceTransmittance.a > 0.999)
	{
		OutLuminance = float4(0.0f, 0.0f, 0.0f, 1.0f);
		return;
	}

	const float CloudDepthKm = VolumetricCloudDepthTexture.Load(int3(PixPos, 0)).r;
	float DeviceZ = CloudDepthKm; // Warning: for simplicity, we use DeviceZ as world distance in kilometer when SAMPLE_ATMOSPHERE_ON_CLOUDS. See special case in IntegrateSingleScatteredLuminance.

#else // SAMPLE_ATMOSPHERE_ON_CLOUDS

#if  MSAA_SAMPLE_COUNT > 1
	float DeviceZ = DepthReadDisabled ? FarDepthValue : MSAADepthTexture.Load(int2(PixPos), SampleIndex).x;
#else
	float DeviceZ = DepthReadDisabled ? FarDepthValue : LookupDeviceZ(UvBuffer);
#endif

	if (DeviceZ == FarDepthValue)
	{
		// Get the light disk luminance to draw 
		LuminanceScale = SkyAtmosphere.SkyLuminanceFactor;
#if SOURCE_DISK_ENABLED
		if (SourceDiskEnabled > 0)
		{
			PreExposedL += GetLightDiskLuminance(WorldPos, WorldDir, 0);
		#if SECOND_ATMOSPHERE_LIGHT_ENABLED
			PreExposedL += GetLightDiskLuminance(WorldPos, WorldDir, 1);
		#endif
		}
#endif

#if RENDERSKY_ENABLED==0
		// We should not render the sky and the current pixels are at far depth, so simply early exit.
		// We enable depth bound when supported to not have to even process those pixels.
		OutLuminance = PrepareOutput(float3(0.0f, 0.0f, 0.0f), float3(1.0f, 1.0f, 1.0f));

		//Now the sky pass can ignore the pixel with depth == far but it will need to alpha clip because not all RHI backend support depthbound tests.
		// And the depthtest is already setup to avoid writing all the pixel closer than to the camera than the start distance (very good optimisation).
		// Since this shader does not write to depth or stencil it should still benefit from EArlyZ even with the clip (See AMD depth-in-depth documentation)
		clip(-1.0f);
		return;
#endif
	}
	else if (SkyAtmosphere.FogShowFlagFactor <= 0.0f)
	{ 
		OutLuminance = PrepareOutput(float3(0.0f, 0.0f, 0.0f), float3(1.0f, 1.0f, 1.0f));
		clip(-1.0f);
		return;
	}
#endif // SAMPLE_ATMOSPHERE_ON_CLOUDS

	float ViewHeight = length(WorldPos);
#if FASTSKY_ENABLED && RENDERSKY_ENABLED
	if (ViewHeight < (Atmosphere.TopRadiusKm * PLANET_RADIUS_RATIO_SAFE_EDGE) && DeviceZ == FarDepthValue
		&& (View.RenderingReflectionCaptureMask > 0.0f || IsSkyAtmosphereRenderedInMain(View.EnvironmentComponentsFlags)))
	{
		float2 UV;

		// The referencial used to build the Sky View lut
		float3x3 LocalReferencial = GetSkyViewLutReferentialParameter();

		// Input vectors expressed in this referencial: Up is always Z. Also note that ViewHeight is unchanged in this referencial.
		float3 WorldPosLocal = float3(0.0, 0.0, ViewHeight);
		float3 UpVectorLocal = float3(0.0, 0.0, 1.0);
		float3 WorldDirLocal = mul(LocalReferencial, WorldDir);
		float ViewZenithCosAngle = dot(WorldDirLocal, UpVectorLocal);

		// Now evaluate inputs in the referential
		bool IntersectGround = RaySphereIntersectNearest(WorldPosLocal, WorldDirLocal, float3(0, 0, 0), Atmosphere.BottomRadiusKm) >= 0.0f;

		SkyViewLutParamsToUv(IntersectGround, ViewZenithCosAngle, WorldDirLocal, ViewHeight, Atmosphere.BottomRadiusKm, SkyAtmosphere.SkyViewLutSizeAndInvSize, UV);
		float4 SkyLuminanceTransmittance = SkyViewLutTexture.SampleLevel(SkyViewLutTextureSampler, UV, 0);
		float3 SkyLuminance = SkyLuminanceTransmittance.rgb;

		float3 SkyGreyTransmittance = 1.0f;
		FLATTEN
		if(bPropagateAlphaNonReflection > 0)
		{
			SkyGreyTransmittance = SkyLuminanceTransmittance.aaa;
		}

		PreExposedL += SkyLuminance * LuminanceScale * (ViewOneOverPreExposure * OutputPreExposure);
	
		OutLuminance = PrepareOutput(PreExposedL, SkyGreyTransmittance);
		UpdateVisibleSkyAlpha(DeviceZ, OutLuminance);
		return;
	}
#endif

#if FASTAERIALPERSPECTIVE_ENABLED

#if COLORED_TRANSMITTANCE_ENABLED
#error The FASTAERIALPERSPECTIVE_ENABLED path does not support COLORED_TRANSMITTANCE_ENABLED.
#else

	float3 DepthBufferTranslatedWorldPos = GetScreenTranslatedWorldPos(SVPos, DeviceZ).xyz;
	float4 NDCPosition = mul(float4(DepthBufferTranslatedWorldPos.xyz, 1), View.TranslatedWorldToClip);

	const float NearFadeOutRangeInvDepthKm = 1.0 / 0.00001f; // 1 centimeter fade region
	float4 AP = GetAerialPerspectiveLuminanceTransmittance(
		ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,
		NDCPosition, (DepthBufferTranslatedWorldPos - GetCameraTranslatedWorldPos()) * CM_TO_SKY_UNIT,
		CameraAerialPerspectiveVolumeTexture, CameraAerialPerspectiveVolumeTextureSampler,
		SkyAtmosphere.CameraAerialPerspectiveVolumeDepthResolutionInv,
		SkyAtmosphere.CameraAerialPerspectiveVolumeDepthResolution,
		AerialPerspectiveStartDepthKm,
		SkyAtmosphere.CameraAerialPerspectiveVolumeDepthSliceLengthKm,
		SkyAtmosphere.CameraAerialPerspectiveVolumeDepthSliceLengthKmInv,
		ViewOneOverPreExposure * OutputPreExposure,
		NearFadeOutRangeInvDepthKm);

	PreExposedL += AP.rgb * LuminanceScale;
	float Transmittance = AP.a;

	OutLuminance = PrepareOutput(PreExposedL, float3(Transmittance, Transmittance, Transmittance));
	UpdateVisibleSkyAlpha(DeviceZ, OutLuminance);
	return;
#endif

#else // FASTAERIALPERSPECTIVE_ENABLED

	// Move to top atmosphere as the starting point for ray marching.
	// This is critical to be after the above to not disrupt above atmosphere tests and voxel selection.
	if (!MoveToTopAtmosphere(WorldPos, WorldDir, Atmosphere.TopRadiusKm))
	{
		// Ray is not intersecting the atmosphere
		OutLuminance = PrepareOutput(PreExposedL);
		return;
	}

#if RENDERSKY_ENABLED
	if (DeviceZ != FarDepthValue)
	{
		// Offset start of integration only if we are going ot hit an opaque pixel (aerial perspective part) and only if we are not going to go past it.
		const float3 DepthBufferTranslatedWorldPos = GetScreenTranslatedWorldPos(SVPos, DeviceZ).xyz;
		if (length(DepthBufferTranslatedWorldPos - GetCameraTranslatedWorldPos()) > AerialPerspectiveStartDepthKm * KILOMETER_TO_CENTIMETER)
		{
			WorldPos += WorldDir * AerialPerspectiveStartDepthKm;
		}
		else
		{
			OutLuminance = PrepareOutput(float3(0.0f, 0.0f, 0.0f), float3(1.0f, 1.0f, 1.0f));
			clip(-1.0f);
			return;
		}
	}
#else
	// Apply the start depth offset after moving to the top of atmosphere for consistency (and to avoid wrong out-of-atmosphere test resulting in black pixels).
	WorldPos += WorldDir * AerialPerspectiveStartDepthKm;
#endif

	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = true;
		Sampling.MinSampleCount = SkyAtmosphere.SampleCountMin;
		Sampling.MaxSampleCount = SkyAtmosphere.SampleCountMax;
		Sampling.DistanceToSampleCountMaxInv = SkyAtmosphere.DistanceToSampleCountMaxInv;
	}
	const bool Ground = false;
	const bool MieRayPhase = true;
	const float AerialPespectiveViewDistanceScale = DeviceZ == FarDepthValue ? 1.0f : SkyAtmosphere.AerialPespectiveViewDistanceScale;
	SingleScatteringResult ss = IntegrateSingleScatteredLuminance(
		SVPos, WorldPos, WorldDir,
		Ground, Sampling, DeviceZ, MieRayPhase,
		View.AtmosphereLightDirection[0].xyz, View.AtmosphereLightDirection[1].xyz, 
		View.AtmosphereLightIlluminanceOuterSpace[0].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor, 
		View.AtmosphereLightIlluminanceOuterSpace[1].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		AerialPespectiveViewDistanceScale);

	PreExposedL += ss.L * LuminanceScale;
	
	if (View.RenderingReflectionCaptureMask == 0.0f && !IsSkyAtmosphereRenderedInMain(View.EnvironmentComponentsFlags))
	{
		PreExposedL = 0.0f;
	}

#if SAMPLE_ATMOSPHERE_ON_CLOUDS
	// We use gray scale transmittance to match the rendering when applying the AerialPerspective texture
	const float GreyScaleAtmosphereTransmittance = dot(ss.Transmittance, float3(1.0 / 3.0f, 1.0 / 3.0f, 1.0 / 3.0f));
	// Reduce cloud luminance according to the atmosphere transmittance and add the atmosphere in scattred luminance according to the cloud coverage.
	PreExposedL = CloudLuminanceTransmittance.rgb * GreyScaleAtmosphereTransmittance + CloudCoverage * PreExposedL;
	// Coverage of the cloud layer itself does not change.
	ss.Transmittance = CloudLuminanceTransmittance.a;
#endif

#if COLORED_TRANSMITTANCE_ENABLED
#error Requires support for dual source blending.
	output.Luminance = float4(PreExposedL, 1.0f);
	output.Transmittance = float4(ss.Transmittance, 1.0f);
#else
	OutLuminance = PrepareOutput(PreExposedL, ss.Transmittance);
	UpdateVisibleSkyAlpha(DeviceZ, OutLuminance);
	return;
#endif

#endif // FASTAERIALPERSPECTIVE_ENABLED
}



////////////////////////////////////////////////////////////
// Transmittance LUT
////////////////////////////////////////////////////////////

#ifndef THREADGROUP_SIZE
#define THREADGROUP_SIZE 1
#endif

// see SkyAtmosphereRendering.cpp GetSkyLutTextureFormat
#if SHADING_PATH_MOBILE
RWTexture2D<float4> TransmittanceLutUAV;
#else
RWTexture2D<float3> TransmittanceLutUAV;
#endif

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RenderTransmittanceLutCS(uint3 ThreadId : SV_DispatchThreadID)
{
	float2 PixPos = float2(ThreadId.xy) + 0.5f;

	// Compute camera position from LUT coords
	float2 UV = (PixPos) * SkyAtmosphere.TransmittanceLutSizeAndInvSize.zw;
	float ViewHeight;
	float ViewZenithCosAngle;
	UvToLutTransmittanceParams(ViewHeight, ViewZenithCosAngle, UV);

	//  A few extra needed constants
	float3 WorldPos = float3(0.0f, 0.0f, ViewHeight);
	float3 WorldDir = float3(0.0f, sqrt(1.0f - ViewZenithCosAngle * ViewZenithCosAngle), ViewZenithCosAngle);

	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = false;
		Sampling.SampleCountIni = SkyAtmosphere.TransmittanceSampleCount;
	}
	const bool Ground = false;
	const float DeviceZ = FarDepthValue;
	const bool MieRayPhase = false;
	const float3 NullLightDirection = float3(0.0f, 0.0f, 1.0f);
	const float3 NullLightIlluminance = float3(0.0f, 0.0f, 0.0f);
	const float AerialPespectiveViewDistanceScale = 1.0f;
	SingleScatteringResult ss = IntegrateSingleScatteredLuminance(
		float4(PixPos,0.0f,1.0f), WorldPos, WorldDir,
		Ground, Sampling, DeviceZ, MieRayPhase,
		NullLightDirection, NullLightDirection, NullLightIlluminance, NullLightIlluminance,
		AerialPespectiveViewDistanceScale);

	float3 transmittance = exp(-ss.OpticalDepth);

	transmittance = EncodeTransmittance(transmittance);

#if SHADING_PATH_MOBILE
	TransmittanceLutUAV[int2(PixPos)] = float4(transmittance, 0.0f);
#else
	TransmittanceLutUAV[int2(PixPos)] = transmittance;
#endif
}



////////////////////////////////////////////////////////////
// Multi-scattering LUT
////////////////////////////////////////////////////////////

// see SkyAtmosphereRendering.cpp GetSkyLutTextureFormat
#if SHADING_PATH_MOBILE
RWTexture2D<float4> MultiScatteredLuminanceLutUAV;
#else
RWTexture2D<float3> MultiScatteredLuminanceLutUAV;
#endif
Buffer<float4> UniformSphereSamplesBuffer;
uint UniformSphereSamplesBufferSampleCount;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RenderMultiScatteredLuminanceLutCS(uint3 ThreadId : SV_DispatchThreadID)
{
	float2 PixPos = float2(ThreadId.xy) + 0.5f;
	// We do no apply UV transform from sub range here as it has minimal impact.

	float CosLightZenithAngle = (PixPos.x * SkyAtmosphere.MultiScatteredLuminanceLutSizeAndInvSize.z) * 2.0f - 1.0f;
	float3 LightDir = float3(0.0f, sqrt(saturate(1.0f - CosLightZenithAngle * CosLightZenithAngle)), CosLightZenithAngle);
	const float3 NullLightDirection = float3(0.0f, 0.0f, 1.0f);
	const float3 NullLightIlluminance = float3(0.0f, 0.0f, 0.0f);
	const float3 OneIlluminance = float3(1.0f, 1.0f, 1.0f);  // Assume a pure white light illuminance for the LUT to act as a transfer (be independent of the light, only dependent on the earth)
	float ViewHeight = Atmosphere.BottomRadiusKm + (PixPos.y * SkyAtmosphere.MultiScatteredLuminanceLutSizeAndInvSize.w) * (Atmosphere.TopRadiusKm - Atmosphere.BottomRadiusKm);

	float3 WorldPos = float3(0.0f, 0.0f, ViewHeight);
	float3 WorldDir = float3(0.0f, 0.0f, 1.0f);

	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = false;
		Sampling.SampleCountIni = SkyAtmosphere.MultiScatteringSampleCount;
	}
	const bool Ground = true;
	const float DeviceZ = FarDepthValue;
	const bool MieRayPhase = false;
	const float AerialPespectiveViewDistanceScale = 1.0f;

	const float SphereSolidAngle = 4.0f * PI;
	const float IsotropicPhase = 1.0f / SphereSolidAngle;

#if HIGHQUALITY_MULTISCATTERING_APPROX_ENABLED

	float3 IntegratedIlluminance = 0.0f;
	float3 MultiScatAs1 = 0.0f;
	for (int s = 0; s < UniformSphereSamplesBufferSampleCount; ++s)
	{
		SingleScatteringResult r0 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, UniformSphereSamplesBuffer[s].xyz, Ground, Sampling, DeviceZ, MieRayPhase,
			LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);

		IntegratedIlluminance += r0.L;
		MultiScatAs1 += r0.MultiScatAs1;
	}
	const float InvCount = 1.0f / float(UniformSphereSamplesBufferSampleCount);
	IntegratedIlluminance *= SphereSolidAngle * InvCount;
	MultiScatAs1 *= InvCount;

	float3 InScatteredLuminance = IntegratedIlluminance * IsotropicPhase;

#elif 1

	// Cheap and good enough approximation (but lose energy) 
	SingleScatteringResult r0 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, WorldDir, Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r1 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, -WorldDir, Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);

	float3 IntegratedIlluminance = (SphereSolidAngle / 2.0f) * (r0.L + r1.L);
	float3 MultiScatAs1 = (1.0f / 2.0f)*(r0.MultiScatAs1 + r1.MultiScatAs1);
	float3 InScatteredLuminance = IntegratedIlluminance * IsotropicPhase;

#else

	// Less cheap but approximation closer to ground truth
	SingleScatteringResult r0 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.70710678118f, 0.0f, 0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r1 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(-0.70710678118f, 0.0f, 0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r2 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.0f, 0.70710678118f, 0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r3 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.0f, -0.70710678118f, 0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r4 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.70710678118f, 0.0f, -0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r5 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(-0.70710678118f, 0.0f, -0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r6 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.0f, 0.70710678118f, -0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);
	SingleScatteringResult r7 = IntegrateSingleScatteredLuminance(float4(PixPos, 0.0f, 1.0f), WorldPos, float3(0.0f, -0.70710678118f, -0.70710678118f), Ground, Sampling, DeviceZ, MieRayPhase,
		LightDir, NullLightDirection, OneIlluminance, NullLightIlluminance, AerialPespectiveViewDistanceScale);

	// Integral of in-scattered Luminance (Lumen/(m2.sr)) over the sphere gives illuminance (Lumen/m2). 
	// This is done with equal importance for each samples over the sphere. 
	float3 IntegratedIlluminance = (SphereSolidAngle / 8.0f) * (r0.L + r1.L + r2.L + r3.L + r4.L + r5.L + r6.L + r7.L);

	// MultiScatAs1 represents the contribution of a uniform environment light over a sphere of luminance 1 and assuming an isotropic phase function 
	float3 MultiScatAs1 = (1.0f / 8.0f)*(r0.MultiScatAs1 + r1.MultiScatAs1 + r2.MultiScatAs1 + r3.MultiScatAs1 + r4.MultiScatAs1 + r5.MultiScatAs1 + r6.MultiScatAs1 + r7.MultiScatAs1);

	// Compute the InScatteredLuminance (Lumen/(m2.sr)) assuming a uniform IntegratedIlluminance, isotropic phase function (1.0/sr)
	// and the fact that this illumiance would be used for each path/raymarch samples of each path
	float3 InScatteredLuminance = IntegratedIlluminance * IsotropicPhase;

#endif 

	// MultiScatAs1 represents the amount of luminance scattered as if the integral of scattered luminance over the sphere would be 1.
	//  - 1st order of scattering: one can ray-march a straight path as usual over the sphere. That is InScatteredLuminance.
	//  - 2nd order of scattering: the inscattered luminance is InScatteredLuminance at each of samples of fist order integration. Assuming a uniform phase function that is represented by MultiScatAs1,
	//  - 3nd order of scattering: the inscattered luminance is (InScatteredLuminance * MultiScatAs1 * MultiScatAs1)
	//  - etc.
#if	MULTI_SCATTERING_POWER_SERIE==0
	float3 MultiScatAs1SQR = MultiScatAs1 * MultiScatAs1;
	float3 L = InScatteredLuminance * (1.0f + MultiScatAs1 + MultiScatAs1SQR + MultiScatAs1 * MultiScatAs1SQR + MultiScatAs1SQR * MultiScatAs1SQR);
#else
	// For a serie, sum_{n=0}^{n=+inf} = 1 + r + r^2 + r^3 + ... + r^n = 1 / (1.0 - r), see https://en.wikipedia.org/wiki/Geometric_series  
	const float3 R = MultiScatAs1;
	const float3 SumOfAllMultiScatteringEventsContribution = 1.0f / (1.0f - R);
	float3 L = InScatteredLuminance * SumOfAllMultiScatteringEventsContribution;
#endif

	// MultipleScatteringFactor can be applied here because the LUT is compute every frame
	// L is pre-exposed since InScatteredLuminance is computed from pre-exposed sun light. So multi-scattering contribution is pre-exposed.
#if SHADING_PATH_MOBILE
	MultiScatteredLuminanceLutUAV[int2(PixPos)] = float4(L * Atmosphere.MultiScatteringFactor, 0.0f);
#else
	MultiScatteredLuminanceLutUAV[int2(PixPos)] = L * Atmosphere.MultiScatteringFactor;
#endif
}



////////////////////////////////////////////////////////////
// Sky View LUT
////////////////////////////////////////////////////////////

// Even thought the texture can have 3 channels only (see SkyAtmosphereRendering.cpp GetSkyLutTextureFormat)
// We always write 4 channels as the grey scale transmittance can be used when alpha propagation is enabled.
RWTexture2D<float4> SkyViewLutUAV;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RenderSkyViewLutCS(uint3 ThreadId : SV_DispatchThreadID)
{
	float2 PixPos = float2(ThreadId.xy) + 0.5f;
	float2 UV = PixPos * SkyAtmosphere.SkyViewLutSizeAndInvSize.zw;

	float3 WorldPos = GetTranslatedCameraPlanetPos();

	// For the sky view lut to work, and not be distorted, we need to transform the view and light directions 
	// into a referential with UP being perpendicular to the ground. And with origin at the planet center.

	// This is the local referencial
	float3x3 LocalReferencial = GetSkyViewLutReferentialParameter();

	// This is the LUT camera height and position in the local referential
	float ViewHeight = length(WorldPos);
	WorldPos = float3(0.0, 0.0, ViewHeight);

	// Get the view direction in this local referential
	float3 WorldDir;
	UvToSkyViewLutParams(WorldDir, ViewHeight, UV);
	// And also both light source direction
	float3 AtmosphereLightDirection0 = View.AtmosphereLightDirection[0].xyz;
	AtmosphereLightDirection0 = mul(LocalReferencial, AtmosphereLightDirection0);
	float3 AtmosphereLightDirection1 = View.AtmosphereLightDirection[1].xyz;
	AtmosphereLightDirection1 = mul(LocalReferencial, AtmosphereLightDirection1);


	// Move to top atmospehre
	if (!MoveToTopAtmosphere(WorldPos, WorldDir, Atmosphere.TopRadiusKm))
	{
		// Ray is not intersecting the atmosphere
		SkyViewLutUAV[int2(PixPos)] = 0.0f;
		return;
	}


	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = true;
		Sampling.MinSampleCount = SkyAtmosphere.FastSkySampleCountMin;
		Sampling.MaxSampleCount = SkyAtmosphere.FastSkySampleCountMax;
		Sampling.DistanceToSampleCountMaxInv = SkyAtmosphere.FastSkyDistanceToSampleCountMaxInv;
	}
	const bool Ground = false;
	const float DeviceZ = FarDepthValue;
	const bool MieRayPhase = true;
	const float AerialPespectiveViewDistanceScale = 1.0f;
	SingleScatteringResult ss = IntegrateSingleScatteredLuminance(
		float4(PixPos, 0.0f, 1.0f), WorldPos, WorldDir,
		Ground, Sampling, DeviceZ, MieRayPhase,
		AtmosphereLightDirection0, AtmosphereLightDirection1,
		View.AtmosphereLightIlluminanceOuterSpace[0].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		View.AtmosphereLightIlluminanceOuterSpace[1].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		AerialPespectiveViewDistanceScale);

	const float Transmittance = dot(ss.Transmittance, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
	SkyViewLutUAV[int2(PixPos)] = float4(ss.L, Transmittance);
}



////////////////////////////////////////////////////////////
// Distant sky light LUT
////////////////////////////////////////////////////////////

// see SkyAtmosphereRendering.cpp GetSkyLutTextureFormat
RWStructuredBuffer<float4> DistantSkyLightLutBufferUAV;
RWBuffer<float4> MobileDistantSkyLightLutBufferUAV;
//Buffer<float4> UniformSphereSamplesBuffer;
float4 AtmosphereLightDirection0;
float4 AtmosphereLightDirection1;
float4 AtmosphereLightIlluminanceOuterSpace0;
float4 AtmosphereLightIlluminanceOuterSpace1;
float DistantSkyLightSampleAltitude;

groupshared float3 GroupSkyLuminanceSamples[THREADGROUP_SIZE*THREADGROUP_SIZE];

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RenderDistantSkyLightLutCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const int LinearIndex = ThreadId.y*THREADGROUP_SIZE + ThreadId.x;
	float2 PixPos = float2(ThreadId.xy) + 0.5f;
	float2 UV = PixPos * SkyAtmosphere.SkyViewLutSizeAndInvSize.zw;

	// As of today, we assume the world is alway at the top of the planet along Z.
	// If it needs to change, we can transform all AtmosphereLightDirection into local basis onder the camera (it would then be view dependent).
	// Overall this is fine because this sky lighting ambient is used for clouds using a dome and it won't be used in such sky views.
	// IF needed later, we could compute illuminance for multiple position on earth in a lat/long texture
	float3 SamplePos = float3(0, 0, Atmosphere.BottomRadiusKm + DistantSkyLightSampleAltitude);
	float ViewHeight = length(SamplePos);

	// We are going to trace 64 times using 64 parallel threads.
	// Result are written in shared memory and prefix sum is applied to integrate the lighting in a single RGB value
	// that can then be used to lit clouds in the sky mesh shader graph.

	// Select a direction for this thread
	const float3 SampleDir = UniformSphereSamplesBuffer[LinearIndex].xyz;

	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = false;
		Sampling.SampleCountIni = 10.0f;
	}
	const bool Ground = false;
	const float DeviceZ = FarDepthValue;
	const bool MieRayPhase = false;
	const float AerialPespectiveViewDistanceScale = 1.0f;
	SingleScatteringResult ss = IntegrateSingleScatteredLuminance(
		float4(PixPos, 0.0f, 1.0f), SamplePos, SampleDir,
		Ground, Sampling, DeviceZ, MieRayPhase,
		AtmosphereLightDirection0.xyz, AtmosphereLightDirection1.xyz, 
		AtmosphereLightIlluminanceOuterSpace0.rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		AtmosphereLightIlluminanceOuterSpace1.rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		AerialPespectiveViewDistanceScale);

	GroupSkyLuminanceSamples[LinearIndex] = ss.L * SkyAtmosphere.SkyLuminanceFactor;

	// Wait for all group threads to be done
	GroupMemoryBarrierWithGroupSync();

	// Now we manually apply prefix sum for a thread group size of 64
#if SKYLIGHT_PASS==1 && THREADGROUP_SIZE!=8
#error This shader only works for a thread group size of 8x8
#endif

	if (LinearIndex < 32)
	{
		GroupSkyLuminanceSamples[LinearIndex] += GroupSkyLuminanceSamples[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		GroupSkyLuminanceSamples[LinearIndex] += GroupSkyLuminanceSamples[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 8)
	{
		GroupSkyLuminanceSamples[LinearIndex] += GroupSkyLuminanceSamples[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 4)
	{
		GroupSkyLuminanceSamples[LinearIndex] += GroupSkyLuminanceSamples[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	// The smallest wave size is 4 on Mali G-71 hardware. So now we can do simple math operations without group sync.

	if (LinearIndex < 2)
	{
		GroupSkyLuminanceSamples[LinearIndex] += GroupSkyLuminanceSamples[LinearIndex + 2];
	}
	if (LinearIndex < 1)
	{
		const float3 AccumulatedLuminanceSamples = GroupSkyLuminanceSamples[LinearIndex] + GroupSkyLuminanceSamples[LinearIndex + 1];
		const float SamplerSolidAngle = 4.0f * PI / float(THREADGROUP_SIZE * THREADGROUP_SIZE);
		const float3 Illuminance = AccumulatedLuminanceSamples * SamplerSolidAngle;
		const float3 UniformPhaseFunction = 1.0f / (4.0f * PI);
		
		// Luminance assuming scattering in a medium with a uniform phase function.
		const float4 OutputResult			= float4(Illuminance * UniformPhaseFunction, 0.0f);
		DistantSkyLightLutBufferUAV[0]		= OutputResult;
		MobileDistantSkyLightLutBufferUAV[0]= OutputResult;
		
		// Since this is ran once per scene, we do not have access to view data (VIEWDATA_AVAILABLE==0). 
		// So this buffer is not pre-exposed today.
	}
}



////////////////////////////////////////////////////////////
// Camera aerial perspective volume LUT
////////////////////////////////////////////////////////////

RWTexture3D<float4> CameraAerialPerspectiveVolumeUAV;
#if SEPARATE_MIE_RAYLEIGH_SCATTERING
RWTexture3D<float4> CameraAerialPerspectiveVolumeMieOnlyUAV;
RWTexture3D<float4> CameraAerialPerspectiveVolumeRayOnlyUAV;
#endif
float RealTimeReflection360Mode;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void RenderCameraAerialPerspectiveVolumeCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (SkyAtmosphere.FogShowFlagFactor <= 0.0f)
	{ 
		CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
	#if SEPARATE_MIE_RAYLEIGH_SCATTERING
		CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
		CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
	#endif
		return;
	}

	float2 PixPos = float2(ThreadId.xy) + 0.5f;
	float2 UV = PixPos  * SkyAtmosphere.CameraAerialPerspectiveVolumeSizeAndInvSize.zw;

	float4 SVPos = float4(View.ViewRectMin.xy + UV * View.ViewSizeAndInvSize.xy, 0.0f, 1.0f);// SV_POS as if resolution was the one from the scene view.
	float3 WorldDir = GetScreenWorldDir(SVPos);
	float3 CamPos = GetTranslatedCameraPlanetPos();
	if (IsOrthoProjection())
	{
		CamPos += GetTranslatedWorldCameraPosFromView(SVPos.xy, true);
	}


	if (RealTimeReflection360Mode)
	{
		float2 UnitUV = FromSubUvsToUnit(UV, SkyAtmosphere.CameraAerialPerspectiveVolumeSizeAndInvSize);

		// Simple lat-long mapping  with with UV.y=sin(ElevationAngle)
		float SinPhi = 2.0f * UnitUV.y - 1.0f;
		float CosPhi = sqrt(1.0f - SinPhi * SinPhi);
		float Theta = 2.0f * PI * UnitUV.x;
		float CosTheta = cos(Theta);
		float SinTheta = sqrt(1.0f - CosTheta * CosTheta) * (Theta > PI ? -1.0f : 1.0f);
		WorldDir = float3(CosTheta * CosPhi, SinTheta * CosPhi, SinPhi);
		WorldDir = normalize(WorldDir);
	}

	float Slice = ((float(ThreadId.z) + 0.5f) * SkyAtmosphere.CameraAerialPerspectiveVolumeDepthResolutionInv); // +0.5 to always have a distance to integrate over
	Slice *= Slice;	// squared distribution
	Slice *= SkyAtmosphere.CameraAerialPerspectiveVolumeDepthResolution;

	float3 RayStartWorldPos = CamPos + AerialPerspectiveStartDepthKm * WorldDir; // Offset according to start depth
	float ViewHeight;


	// Compute position from froxel information
	float tMax = Slice * SkyAtmosphere.CameraAerialPerspectiveVolumeDepthSliceLengthKm;
	float3 VoxelWorldPos = RayStartWorldPos + tMax * WorldDir;
	float VoxelHeight = length(VoxelWorldPos);

	// Check if the voxel is under the horizon.
	const float UnderGround = VoxelHeight < Atmosphere.BottomRadiusKm;

	// Check if the voxel is beind the planet (to next check for below the horizon case)
	float3 CameraToVoxel = VoxelWorldPos - CamPos;
	float CameraToVoxelLen = length(CameraToVoxel);
	float3 CameraToVoxelDir = CameraToVoxel / CameraToVoxelLen;
	float PlanetNearT = RaySphereIntersectNearest(CamPos, CameraToVoxelDir, float3(0, 0, 0), Atmosphere.BottomRadiusKm);
	bool BelowHorizon = PlanetNearT > 0.0f && CameraToVoxelLen > PlanetNearT;

	if (BelowHorizon || UnderGround)
	{
		CamPos += normalize(CamPos) * 0.02f;	// TODO: investigate why we need this workaround. Without it, we get some bad color and flickering on the ground only (floating point issue with sphere intersection code?).

		float3 VoxelWorldPosNorm = normalize(VoxelWorldPos);
		float3 CamProjOnGround = normalize(CamPos) * Atmosphere.BottomRadiusKm;
		float3 VoxProjOnGround = VoxelWorldPosNorm * Atmosphere.BottomRadiusKm;
		float3 VoxelGroundToRayStart = CamPos - VoxProjOnGround;
		if (BelowHorizon && dot(normalize(VoxelGroundToRayStart), VoxelWorldPosNorm) < 0.0001f)
		{
			// We are behind the sphere and the sphere normal is pointing away from V: we are below the horizon.
			float3 MiddlePoint = 0.5f * (CamProjOnGround + VoxProjOnGround);
			float MiddlePointHeight = length(MiddlePoint);

			// Compute the new position to evaluate and store the value in the voxel.
			// the position is the oposite side of the horizon point from the view point,
			// The offset of 1.001f is needed to get matching colors and for the ray to not hit the earth again later due to floating point accuracy
			float3 MiddlePointOnGround = normalize(MiddlePoint) * Atmosphere.BottomRadiusKm;// *1.001f;
			VoxelWorldPos = CamPos + 2.0f * (MiddlePointOnGround - CamPos);

			//CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(1, 0, 0, 0);
			//#if SEPARATE_MIE_RAYLEIGH_SCATTERING
			//	CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId] = float4(1, 0, 0, 0);
			//	CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId] = float4(1, 0, 0, 0);
			//#endif
			//return; // debug
		}
		else if (UnderGround)
		{
			//No obstruction from the planet, so use the point on the ground
			VoxelWorldPos = normalize(VoxelWorldPos) * (Atmosphere.BottomRadiusKm);
			//VoxelWorldPos = CamPos + CameraToVoxelDir * PlanetNearT;		// better match but gives visual artefact as visible voxels on a simple plane at altitude 0

			//CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(0, 1, 0, 0);
			//#if SEPARATE_MIE_RAYLEIGH_SCATTERING
			//	CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId] = float4(0, 1, 0, 0);
			//	CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId] = float4(0, 1, 0, 0);
			//#endif
			//return; // debug
		}
		 
		WorldDir = normalize(VoxelWorldPos - CamPos);
		RayStartWorldPos = CamPos + AerialPerspectiveStartDepthKm * WorldDir; // Offset according to start depth
		tMax = length(VoxelWorldPos - RayStartWorldPos);
	}
	float tMaxMax = tMax;

	// Move ray marching start up to top atmosphere.
	ViewHeight = length(RayStartWorldPos);
	if (ViewHeight >= Atmosphere.TopRadiusKm)
	{
		float3 prevWorlPos = RayStartWorldPos;
		if (!MoveToTopAtmosphere(RayStartWorldPos, WorldDir, Atmosphere.TopRadiusKm))
		{
			// Ray is not intersecting the atmosphere
			CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
		#if SEPARATE_MIE_RAYLEIGH_SCATTERING
			CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
			CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
		#endif
			return;
		}
		float LengthToAtmosphere = length(prevWorlPos - RayStartWorldPos);
		if (tMaxMax < LengthToAtmosphere)
		{
			// tMaxMax for this voxel is not within the planet atmosphere
			CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
		#if SEPARATE_MIE_RAYLEIGH_SCATTERING
			CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
			CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId] = float4(0.0f, 0.0f, 0.0f, 1.0f);
		#endif
			return;
		}
		// Now world position has been moved to the atmosphere boundary: we need to reduce tMaxMax accordingly. 
		tMaxMax = max(0.0, tMaxMax - LengthToAtmosphere);
	}


	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = false;
		Sampling.SampleCountIni = max(1.0f, (float(ThreadId.z) + 1.0f) * SkyAtmosphere.CameraAerialPerspectiveSampleCountPerSlice);
	}
	const bool Ground = false;
	const float DeviceZ = FarDepthValue;
	const bool MieRayPhase = true;
	const float AerialPespectiveViewDistanceScale = SkyAtmosphere.AerialPespectiveViewDistanceScale;
	SingleScatteringResult ss = IntegrateSingleScatteredLuminance( 
		float4(PixPos, 0.0f, 1.0f), RayStartWorldPos, WorldDir,
		Ground, Sampling, DeviceZ, MieRayPhase,
		View.AtmosphereLightDirection[0].xyz, View.AtmosphereLightDirection[1].xyz, 
		View.AtmosphereLightIlluminanceOuterSpace[0].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		View.AtmosphereLightIlluminanceOuterSpace[1].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor,
		AerialPespectiveViewDistanceScale,
		tMaxMax);

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags) && !RealTimeReflection360Mode)
	{
		ss.L *= 0;
		ss.LMieOnly *= 0;
		ss.LRayOnly *= 0;
	}
#endif

	const float Transmittance = dot(ss.Transmittance, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
	CameraAerialPerspectiveVolumeUAV[ThreadId] = float4(ss.L, Transmittance);
#if SEPARATE_MIE_RAYLEIGH_SCATTERING
		const float TransmittanceMieOnly = dot(ss.TransmittanceMieOnly, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
		const float TransmittanceRayOnly = dot(ss.TransmittanceRayOnly, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
		CameraAerialPerspectiveVolumeMieOnlyUAV[ThreadId]	= float4(ss.LMieOnly, TransmittanceMieOnly);
		CameraAerialPerspectiveVolumeRayOnlyUAV[ThreadId]	= float4(ss.LRayOnly, TransmittanceRayOnly);
#endif
}



////////////////////////////////////////////////////////////
// Debug
////////////////////////////////////////////////////////////

float ViewPortWidth;
float ViewPortHeight;

float Mean3(float3 v)
{
	return dot(v, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
}

float3 SimpleTonemap(float3 Luminance)
{
	return pow(Luminance, 1.0 / 2.2);
}

void RenderSkyAtmosphereDebugPS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminance : SV_Target0)
{
	float2 PixPos = SVPos.xy;

	// Common ray marching input
	SamplingSetup Sampling = (SamplingSetup)0;
	{
		Sampling.VariableSampleCount = false;
		Sampling.SampleCountIni = 128.0f;
	}
	const bool Ground = false;
	const bool MieRayPhase = true;
	float DeviceZ = FarDepthValue;
	const float3 NullLightDirection = float3(0.0f, 0.0f, 1.0f);
	const float3 NullLightIlluminance = float3(0.0f, 0.0f, 0.0f);
	const float AerialPespectiveViewDistanceScale = 1.0f;

	// We position the camera at the same height as the view port camera, along z-up vector
	float3 WorldPos = float3(0.0f, 0.0f, length((View.TranslatedWorldCameraOrigin - GetSkyPlanetTranslatedWorldCenterAndViewHeightParameter()) * CM_TO_SKY_UNIT));

	// Some organisation constants
	const float Margin = 2.0f;
	const float TimeOfDayHeight = 64.0f;
	const float TimeOfDayTop = TimeOfDayHeight + Margin*2.0f;

	// Hemisphere view, also show a bit of ground.
	float2 DebugSize = ViewPortWidth * 0.25f;
	float2 DebugPos = float2(2.0f, ViewPortHeight - DebugSize.y - TimeOfDayTop);
	if(all(PixPos < (DebugPos+DebugSize)) && all(PixPos > DebugPos))
	{
		float2 UV = clamp(PixPos - DebugPos, 0.0f, DebugSize) / DebugSize;

		UV = UV * 2.0f - 1.0f;
		float UVLen = length(UV);
		if (UVLen < 1.0f)
		{
			float SinToHorizon = -2.0f * UVLen*UVLen + 1.0f;
			float CosXY = cos(asin(SinToHorizon));
			float3 WorldDir = normalize(float3(CosXY*normalize(UV), SinToHorizon));

			SingleScatteringResult SS = IntegrateSingleScatteredLuminance(
				SVPos, WorldPos, WorldDir,
				Ground, Sampling, DeviceZ, MieRayPhase,
				View.AtmosphereLightDirection[0].xyz, NullLightDirection, 
				View.AtmosphereLightIlluminanceOuterSpace[0].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor, 
				NullLightIlluminance,
				AerialPespectiveViewDistanceScale);
			OutLuminance = float4(SimpleTonemap(SS.L), 0.0f);
			return;
		}
		OutLuminance = 0.0f;
		return;
	}

	// All-at-once time of day visualization.
	DebugSize = float2(1024.0f, TimeOfDayHeight);
	DebugPos = float2(70.0f, ViewPortHeight - DebugSize.y - Margin);
	// Adaptive TimeOfDay count
	const float DesiredTimeOfDayWidth = 128.0f;
	float NumTimeOfDay = floor((ViewPortWidth - DebugPos.x) / DesiredTimeOfDayWidth);
	DebugSize.x = NumTimeOfDay * DesiredTimeOfDayWidth;
	if (all(PixPos < (DebugPos + DebugSize)) && all(PixPos > DebugPos))
	{
		float2 UV = clamp(PixPos - DebugPos, 0.0f, DebugSize) / DebugSize;

		float CurrentTimeOfDay = floor(UV.x*NumTimeOfDay) / NumTimeOfDay;
		UV.x = frac(UV.x*NumTimeOfDay);

		float LightHorizonAngle = -0.05f + CurrentTimeOfDay * CurrentTimeOfDay * PI * 0.5f;
		float3 LightDir = float3(cos(LightHorizonAngle), 0.0f, sin(LightHorizonAngle));

		float ViewLightAngleCos = cos(PI * UV.x);
		float ViewLightAngleSin = sqrt(1.0f - ViewLightAngleCos * ViewLightAngleCos);
		float ViewHorizonCos = cos(0.5f * PI * (1.0f-UV.y));
		float ViewHorizonSin = sqrt(1.0f - ViewHorizonCos * ViewHorizonCos);

		float3 WorldDir = float3( 
			ViewHorizonCos * ViewLightAngleCos,
			ViewHorizonCos * ViewLightAngleSin,
			ViewHorizonSin );

		SingleScatteringResult SS = IntegrateSingleScatteredLuminance(
			SVPos, WorldPos, WorldDir,
			Ground, Sampling, DeviceZ, MieRayPhase,
			LightDir, NullLightDirection, 
			View.AtmosphereLightIlluminanceOuterSpace[0].rgb * SkyAtmosphere.SkyAndAerialPerspectiveLuminanceFactor, 
			NullLightIlluminance,
			AerialPespectiveViewDistanceScale);

		OutLuminance = float4(SimpleTonemap(SS.L), 0.0f);
		return;
	}
	 
	// Atmosphere density distribution.
	DebugSize = float2(ViewPortWidth * 0.2f, ViewPortHeight * 0.7f);
	DebugPos = float2(ViewPortWidth - DebugSize.x - 2.0f, ViewPortHeight * 0.1f);
	if (all(PixPos < (DebugPos + DebugSize)) && all(PixPos > DebugPos))
	{
		float2 UV = clamp(PixPos - DebugPos, 0.0f, DebugSize) / DebugSize;
		UV.y = 1.0f - UV.y;
		UV = UV.yx;

		float3 Color = float3(pow(1.0f-UV, 200.0f), 0.0f);

		const float TotMax = 0.01f;	// Cosntant max extinction is less confusing



		float3 SampleWorldPos = float3(0.0f, 0.0f, Atmosphere.BottomRadiusKm + UV.x * (Atmosphere.TopRadiusKm - Atmosphere.BottomRadiusKm));
		MediumSampleRGB Sample = SampleAtmosphereMediumRGB(SampleWorldPos);

		// Curves represent the maximum contribution as extinction (no differenciation between scattering or absorption)
		float3 Curves = float3(
			Mean3(Sample.ExtinctionRay) / TotMax,
			Mean3(Sample.ExtinctionMie) / TotMax,
			Mean3(Sample.ExtinctionOzo) / TotMax
			);

		Curves = saturate((Curves - UV.y)*100000.0f);

		OutLuminance = float4(Curves, 0.0f); // No exposure for constant visual
		return;
	}

	// Brown ground
	DebugSize.y += 15.0f;
	if (all(PixPos < (DebugPos + DebugSize)) && all(PixPos > DebugPos))
	{
		OutLuminance = float4(float3(0.125f, 0.05f, 0.005f), 0.0f); // No exposure for constant visual
		return;
	}

	clip(-1.0f);
	OutLuminance = float4(0.0f, 0.0f, 0.0f, 1.0f);
	return;
}



////////////////////////////////////////////////////////////
// Editor
////////////////////////////////////////////////////////////

#ifdef SHADER_EDITOR_HUD

#include "MiniFontCommon.ush"

void RenderSkyAtmosphereEditorHudPS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminance : SV_Target0)
{
	int2 PixelPos = SVPos.xy;
	PixelPos = PixelPos % int2(512, 256);

	if (!(PixelPos.x < 440 && PixelPos.y < 30))
	{
		clip(-1.0);
		OutLuminance = float4(1,0,0, 1);
		return;
	}

	const int StartX = 10;
	int2 Cursor = int2(StartX, 10);
	float3 OutColor = 0.0f;

	const float3 TextColor = float3(1, 0.75, 0.25);
#define P(x)  PrintCharacter(PixelPos, OutColor, TextColor, Cursor, x);
#define SPACE PrintCharacter(PixelPos, OutColor, TextColor, Cursor, _SPC_);

	P(_Y_)P(_O_)P(_U_)P(_R_)
	SPACE
	P(_S_)P(_C_)P(_E_)P(_N_)P(_E_)
	SPACE
	P(_C_)P(_O_)P(_N_)P(_T_)P(_A_)P(_I_)P(_N_)P(_S_)
	SPACE
	P(_A_)
	SPACE
	P(_S_)P(_K_)P(_Y_)P(_D_)P(_O_)P(_M_)P(_E_)
	SPACE
	P(_M_)P(_E_)P(_S_)P(_H_)
	SPACE
	P(_W_)P(_I_)P(_T_)P(_H_)
	SPACE
	P(_A_)
	SPACE
	P(_S_)P(_K_)P(_Y_)
	SPACE
	P(_M_)P(_A_)P(_T_)P(_E_)P(_R_)P(_I_)P(_A_)P(_L_)P(_COMMA_)

	Cursor.x = StartX;
	Cursor.y += 12;
		
	P(_B_)P(_U_)P(_T_)
	SPACE
	P(_I_)P(_T_)
	SPACE
	P(_D_)P(_O_)P(_E_)P(_S_)
	SPACE
	P(_N_)P(_O_)P(_T_)
	SPACE
	P(_C_)P(_O_)P(_V_)P(_E_)P(_R_)
	SPACE
	P(_T_)P(_H_)P(_A_)P(_T_)
	SPACE
	P(_P_)P(_A_)P(_R_)P(_T_)
	SPACE
	P(_O_)P(_F_)
	SPACE
	P(_T_)P(_H_)P(_E_)
	SPACE
	P(_S_)P(_C_)P(_R_)P(_E_)P(_E_)P(_N_)P(_DOT_)P(_DOT_)P(_DOT_)
	SPACE

#undef P
#undef SPACE

	OutLuminance = float4(OutColor,1);
	return;
}

#endif


