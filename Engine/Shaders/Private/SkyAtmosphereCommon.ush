// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyAtmosphereCommon.usf: Sky and atmosphere common functions, 
	e.g. shared with mesh rendering or path tracing.
=============================================================================*/

#pragma once

#include "ParticipatingMediaCommon.ush"
#include "/Engine/Shared/EnvironmentComponentsFlags.h"

// Disabled for all mobile platforms, see USkyLightComponent::IsRealTimeCaptureEnabled
#define AP_SUPPORT_REALTIME_REFLECTION_CAPTURE (FEATURE_LEVEL > FEATURE_LEVEL_ES3_1)

// Controlls Atmosphere. struct access as not all shaders have access to thi
#ifndef AP_SUPPORT_SAMPLE_ATMOSHPERE
	#define AP_SUPPORT_SAMPLE_ATMOSHPERE 1
#endif

// The constants below should match the one in SceneRendering.cpp
// Kilometers as unit for computations related to the sky and its atmosphere
#define CM_TO_SKY_UNIT 0.00001f
#define SKY_UNIT_TO_CM (1.0f/CM_TO_SKY_UNIT)
// Float accuracy offset in Sky unit (km, so this is 1m). Should match the one in FAtmosphereSetup::ComputeViewData
#define PLANET_RADIUS_OFFSET 0.001f

// Planet radius safe edge to make sure ray does intersect with the atmosphere, for it to traverse the atmosphere. Must match the one in FSceneRenderer::RenderSkyAtmosphereInternal.
// This is (0.01km/6420km).
#define PLANET_RADIUS_RATIO_SAFE_EDGE 1.00000155763f


// The number of killometer per slice in the aerial pespective camera volume texture. (assuming a uniform depth distribution)
#define AP_KM_PER_SLICE 4.0f
#define AP_KM_PER_SLICE_INV (1.0f / AP_KM_PER_SLICE)


float2 FromUnitToSubUvs(float2 uv, float4 SizeAndInvSize) { return (uv + 0.5f * SizeAndInvSize.zw) * (SizeAndInvSize.xy / (SizeAndInvSize.xy + 1.0f)); }
float2 FromSubUvsToUnit(float2 uv, float4 SizeAndInvSize) { return (uv - 0.5f * SizeAndInvSize.zw) * (SizeAndInvSize.xy / (SizeAndInvSize.xy - 1.0f)); }


float4 GetAerialPerspectiveLuminanceTransmittance(
	bool ViewIsRealTimeReflectionCapture, float4 CameraAerialPerspectiveVolumeSizeAndInvSize,
	float4 NDC, float3 WorldPositionRelativeToCamera,
	Texture3D<float4> AerialPerspectiveVolumeTexture, SamplerState AerialPerspectiveVolumeTextureSampler,
	float AerialPerspectiveVolumeDepthResolutionInv,
	float AerialPerspectiveVolumeDepthResolution,
	float AerialPerspectiveVolumeStartDepth,
	float AerialPerspectiveVolumeDepthSliceLengthKm,
	float AerialPerspectiveVolumeDepthSliceLengthKmInv,
	float OneOverExposure,
	float NearFadeOutRangeInvDepthKm)
{
	if (View.RenderingReflectionCaptureMask == 0.0f && !IsSkyAtmosphereRenderedInMain(View.EnvironmentComponentsFlags))
	{
		return float4(0.0f, 0.0f, 0.0f, 1.0f);
	}

	if (IsOrthoProjection())
	{
		float2 SvPosXY = (((NDC.xy * float2(0.5f, -0.5f)) + 0.5f) * View.ViewSizeAndInvSize.xy) + ResolvedView.ViewRectMin.xy;		
		WorldPositionRelativeToCamera += (GetTranslatedWorldCameraPosFromView(SvPosXY, true) * CM_TO_SKY_UNIT) + WorldPositionRelativeToCamera;
	}
	
	float2 ScreenUv = (NDC.xy / NDC.ww) * float2(0.5f, -0.5f) + 0.5f;

	float tDepth = max(0.0f, length(WorldPositionRelativeToCamera) - AerialPerspectiveVolumeStartDepth);

	float LinearSlice = tDepth * AerialPerspectiveVolumeDepthSliceLengthKmInv;
	float LinearW = LinearSlice * AerialPerspectiveVolumeDepthResolutionInv; // Depth slice coordinate in [0,1]
	float NonLinW = sqrt(LinearW); // Squared distribution
	float NonLinSlice = NonLinW * AerialPerspectiveVolumeDepthResolution;

	const float HalfSliceDepth = 0.70710678118654752440084436210485f; // sqrt(0.5f)
	float Weight = 1.0f;
	if (NonLinSlice < HalfSliceDepth)
	{
		// We multiply by weight to fade to 0 at depth 0. It works for luminance and opacity.
		Weight = saturate(NonLinSlice*NonLinSlice * 2.0f); // Square to have a linear falloff from the change of distribution above
	}
	Weight *= saturate(tDepth * NearFadeOutRangeInvDepthKm);

#if AP_SUPPORT_REALTIME_REFLECTION_CAPTURE
	if (ViewIsRealTimeReflectionCapture)
	{
		// We modify ScreenUv to sample the correct AP accordign to 360 reflection AP LUT.
		// This is the inverse of what is in RenderCameraAerialPerspectiveVolumeCS.
		float3 WorldDir = normalize(WorldPositionRelativeToCamera);

		float SinPhi = WorldDir.z;
		float CosPhi = sqrt(1.0f - SinPhi * SinPhi);
		ScreenUv.y = WorldDir.z * 0.5f + 0.5f;

		float CosTheta = WorldDir.x / CosPhi;
		float SinTheta = WorldDir.y / CosPhi;
		float Theta = acos(CosTheta);
		Theta = SinTheta < 0.0f ? (PI-Theta) + PI : Theta;
		ScreenUv.x = Theta / (2.0 * PI);

		ScreenUv = FromUnitToSubUvs(ScreenUv, CameraAerialPerspectiveVolumeSizeAndInvSize);
	}
#endif

	float4 AP = Texture3DSampleLevel(AerialPerspectiveVolumeTexture, AerialPerspectiveVolumeTextureSampler, float3(ScreenUv, NonLinW), 0.0f);

	// Lerp to no contribution near the camera (careful as AP contains transmittance)
	AP.rgb *= Weight;
	AP.a = 1.0 - (Weight * (1.0f - AP.a));

	// Debug Slices
#if 0
	AP.rgba *= frac(clamp(NonLinSlice, 0, AerialPerspectiveVolumeDepthResolution));
	AP.r += LinearW <= 0.0f ? 0.5f : 0.0f;
	AP.g += LinearW >= 1.0f ? 0.5f : 0.0f;
	AP.b += Weight  <  1.0f ? 0.2f+0.2f*Weight : 0.0f;
#endif

	AP.rgb *= OneOverExposure;

	return AP;
}

float4 GetAerialPerspectiveLuminanceTransmittanceWithFogOver(
	bool ViewIsRealTimeReflectionCapture, float4 CameraAerialPerspectiveVolumeSizeAndInvSize,
	float4 NDC, float3 WorldPositionRelativeToCamera,
	Texture3D<float4> AerialPerspectiveVolumeTexture, SamplerState AerialPerspectiveVolumeTextureSampler,
	float AerialPerspectiveVolumeDepthResolutionInv,
	float AerialPerspectiveVolumeDepthResolution,
	float AerialPerspectiveVolumeStartDepth,
	float AerialPerspectiveVolumeDepthSliceLengthKm,
	float AerialPerspectiveVolumeDepthSliceLengthKmInv,
	float OneOverExposure, float4 FogToApplyOver)
{
	const float NearFadeOutRangeInvDepthKm = 1.0 / 0.00001f; // 1 centimeter fade region
	float4 AP = GetAerialPerspectiveLuminanceTransmittance(
		ViewIsRealTimeReflectionCapture, CameraAerialPerspectiveVolumeSizeAndInvSize,
		NDC, WorldPositionRelativeToCamera,
		AerialPerspectiveVolumeTexture, AerialPerspectiveVolumeTextureSampler,
		AerialPerspectiveVolumeDepthResolutionInv,
		AerialPerspectiveVolumeDepthResolution,
		AerialPerspectiveVolumeStartDepth,
		AerialPerspectiveVolumeDepthSliceLengthKm,
		AerialPerspectiveVolumeDepthSliceLengthKmInv,
		OneOverExposure,
		NearFadeOutRangeInvDepthKm);

	float4 FinalFog;
	// Apply any other fog OVER aerial perspective because AP is usually optically thiner.
	FinalFog.rgb = FogToApplyOver.rgb + AP.rgb * FogToApplyOver.a;
	// And combine both transmittance.
	FinalFog.a   = FogToApplyOver.a * AP.a;

	return FinalFog;
}

void fromTransmittanceLutUVs(
	out float ViewHeight, out float ViewZenithCosAngle,
	in float BottomRadius, in float TopRadius, in float2 UV)
{
	float Xmu = UV.x;
	float Xr = UV.y;

	float H = sqrt(TopRadius * TopRadius - BottomRadius * BottomRadius);
	float Rho = H * Xr;
	ViewHeight = sqrt(Rho * Rho + BottomRadius * BottomRadius);

	float Dmin = TopRadius - ViewHeight;
	float Dmax = Rho + H;
	float D = Dmin + Xmu * (Dmax - Dmin);
	ViewZenithCosAngle = D == 0.0f ? 1.0f : (H * H - Rho * Rho - D * D) / (2.0f * ViewHeight * D);
	ViewZenithCosAngle = clamp(ViewZenithCosAngle, -1.0f, 1.0f);
}


void getTransmittanceLutUvs(
	in float viewHeight, in float viewZenithCosAngle, in float BottomRadius, in float TopRadius,
	out float2 UV)
{
	float H = sqrt(max(0.0f, TopRadius * TopRadius - BottomRadius * BottomRadius));
	float Rho = sqrt(max(0.0f, viewHeight * viewHeight - BottomRadius * BottomRadius));

	float Discriminant = viewHeight * viewHeight * (viewZenithCosAngle * viewZenithCosAngle - 1.0f) + TopRadius * TopRadius;
	float D = max(0.0f, (-viewHeight * viewZenithCosAngle + sqrt(Discriminant))); // Distance to atmosphere boundary

	float Dmin = TopRadius - viewHeight;
	float Dmax = Rho + H;
	float Xmu = (D - Dmin) / (Dmax - Dmin);
	float Xr = Rho / H;

	UV = float2(Xmu, Xr);
	//UV = float2(fromUnitToSubUvs(UV.x, TRANSMITTANCE_TEXTURE_WIDTH), fromUnitToSubUvs(UV.y, TRANSMITTANCE_TEXTURE_HEIGHT)); // No real impact so off
}

void SkyViewLutParamsToUv(
	in bool IntersectGround, in float ViewZenithCosAngle, in float3 ViewDir, in float ViewHeight, in float BottomRadius, in float4 SkyViewLutSizeAndInvSize,
	out float2 UV)
{
	float Vhorizon = sqrt(ViewHeight * ViewHeight - BottomRadius * BottomRadius);
	float CosBeta = Vhorizon / ViewHeight;				// GroundToHorizonCos
	float Beta = acosFast4(CosBeta);
	float ZenithHorizonAngle = PI - Beta;
	float ViewZenithAngle = acosFast4(ViewZenithCosAngle);

	if (!IntersectGround)
	{
		float Coord = ViewZenithAngle / ZenithHorizonAngle;
		Coord = 1.0f - Coord;
		Coord = sqrt(Coord);
		Coord = 1.0f - Coord;
		UV.y = Coord * 0.5f;
	}
	else
	{
		float Coord = (ViewZenithAngle - ZenithHorizonAngle) / Beta;
		Coord = sqrt(Coord);
		UV.y = Coord * 0.5f + 0.5f;
	}

	{
		UV.x = (atan2Fast(-ViewDir.y, -ViewDir.x) + PI) / (2.0f * PI);
	}

	// Constrain uvs to valid sub texel range (avoid zenith derivative issue making LUT usage visible)
	UV = FromUnitToSubUvs(UV, SkyViewLutSizeAndInvSize);
}

float3x3 GetSkyViewLutReferential(in float4x4 ViewSkyViewLutReferential)
{
	float3x3 SkyViewLutReferential = (float3x3) ViewSkyViewLutReferential;
	return SkyViewLutReferential;
}

float3 EncodeTransmittance(float3 Transmittance)
{
	// We encode transmittance to avoid banding issue when storing it in RGB8.
	// Encoding is simple square root which is a good fit to get more precision in the low value part,
	// typically at horizon where it is important to get more precision for sunset transmittance on the sun disk.
	return sqrt(Transmittance);
}

float3 DecodeTransmittance(float3 Transmittance)
{
	return Transmittance * Transmittance;
}

float3 GetAtmosphereTransmittance(
	float3 PlanetCenterToWorldPos, float3 WorldDir, float BottomRadius, float TopRadius,
	Texture2D<float4> TransmittanceLutTexture, SamplerState TransmittanceLutTextureSampler)
{
	// For each view height entry, transmittance is only stored from zenith to horizon. Earth shadow is not accounted for.
	// It does not contain earth shadow in order to avoid texel linear interpolation artefact when LUT is low resolution.
	// As such, at the most shadowed point of the LUT when close to horizon, pure black with earth shadow is never hit.
	// That is why we analytically compute the virtual planet shadow here.
	const float2 Sol = RayIntersectSphere(PlanetCenterToWorldPos, WorldDir, float4(float3(0.0f, 0.0f, 0.0f), BottomRadius));
	if (Sol.x > 0.0f || Sol.y > 0.0f)
	{
		return 0.0f;
	}

	const float PHeight = length(PlanetCenterToWorldPos);
	const float3 UpVector = PlanetCenterToWorldPos / PHeight;
	const float LightZenithCosAngle = dot(WorldDir, UpVector);
	float2 TransmittanceLutUv;
	getTransmittanceLutUvs(PHeight, LightZenithCosAngle, BottomRadius, TopRadius, TransmittanceLutUv);
	const float3 TransmittanceToLight = Texture2DSampleLevel(TransmittanceLutTexture, TransmittanceLutTextureSampler, TransmittanceLutUv, 0.0f).rgb;
	return DecodeTransmittance(TransmittanceToLight);
}

float3 GetLightDiskLuminance(
	float3 PlanetCenterToWorldPos, float3 WorldDir, float BottomRadius, float TopRadius,
	Texture2D<float4> TransmittanceLutTexture, SamplerState TransmittanceLutTextureSampler,
	float3 AtmosphereLightDirection, float AtmosphereLightDiscCosHalfApexAngle, float3 AtmosphereLightDiscLuminance)
{
	const float ViewDotLight = dot(WorldDir, AtmosphereLightDirection);
	const float CosHalfApex = AtmosphereLightDiscCosHalfApexAngle;
	if (ViewDotLight > CosHalfApex)
	{
		const float3 TransmittanceToLight = GetAtmosphereTransmittance(
			PlanetCenterToWorldPos, WorldDir, BottomRadius, TopRadius, TransmittanceLutTexture, TransmittanceLutTextureSampler);

		// Soften out the sun disk to avoid bloom flickering at edge. The soften is applied on the outer part of the disk.
		const float SoftEdge = saturate(2.0f * (ViewDotLight - CosHalfApex) / (1.0f - CosHalfApex));

		return TransmittanceToLight * AtmosphereLightDiscLuminance * SoftEdge;
	}
	return 0.0f;
}

#if AP_SUPPORT_SAMPLE_ATMOSHPERE
////////////////////////////////////////////////////////////
// Participating medium properties
////////////////////////////////////////////////////////////

float3 GetAlbedo(float3 Scattering, float3 Extinction)
{
	return Scattering / max(0.001f, Extinction);
}

struct MediumSampleRGB
{
	float3 Scattering;
	float3 Absorption;
	float3 Extinction;

	float3 ScatteringMie;
	float3 AbsorptionMie;
	float3 ExtinctionMie;

	float3 ScatteringRay;
	float3 AbsorptionRay;
	float3 ExtinctionRay;

	float3 ScatteringOzo;
	float3 AbsorptionOzo;
	float3 ExtinctionOzo;

	float3 Albedo;
};

// If this is changed, please also update USkyAtmosphereComponent::GetTransmittance 
MediumSampleRGB SampleAtmosphereMediumRGB(in float3 WorldPos)
{
	const float SampleHeight = max(0.0, (length(WorldPos) - Atmosphere.BottomRadiusKm));

	const float DensityMie = exp(Atmosphere.MieDensityExpScale * SampleHeight);

	const float DensityRay = exp(Atmosphere.RayleighDensityExpScale * SampleHeight);

	const float DensityOzo = SampleHeight < Atmosphere.AbsorptionDensity0LayerWidth ?
		saturate(Atmosphere.AbsorptionDensity0LinearTerm * SampleHeight + Atmosphere.AbsorptionDensity0ConstantTerm) :	// We use saturate to allow the user to create plateau, and it is free on GCN.
		saturate(Atmosphere.AbsorptionDensity1LinearTerm * SampleHeight + Atmosphere.AbsorptionDensity1ConstantTerm);

	MediumSampleRGB s;

	s.ScatteringMie = DensityMie * Atmosphere.MieScattering.rgb;
	s.AbsorptionMie = DensityMie * Atmosphere.MieAbsorption.rgb;
	s.ExtinctionMie = DensityMie * Atmosphere.MieExtinction.rgb;

	s.ScatteringRay = DensityRay * Atmosphere.RayleighScattering.rgb;
	s.AbsorptionRay = 0.0f;
	s.ExtinctionRay = s.ScatteringRay + s.AbsorptionRay;

	s.ScatteringOzo = 0.0f;
	s.AbsorptionOzo = DensityOzo * Atmosphere.AbsorptionExtinction.rgb;
	s.ExtinctionOzo = s.ScatteringOzo + s.AbsorptionOzo;

	s.Scattering = s.ScatteringMie + s.ScatteringRay + s.ScatteringOzo;
	s.Absorption = s.AbsorptionMie + s.AbsorptionRay + s.AbsorptionOzo;
	s.Extinction = s.ExtinctionMie + s.ExtinctionRay + s.ExtinctionOzo;
	s.Albedo = GetAlbedo(s.Scattering, s.Extinction);

	return s;
}
#endif //AP_SUPPORT_SAMPLE_ATMOSHPERE

float3 GetViewDistanceSkyLightColor()
{
#if SHADING_PATH_MOBILE && VERTEXSHADER
	// Some mobile platforms cannot sample structured buffer in vertex shaders (height fog) so in this case a regular buffer is used.
	return View.MobileDistantSkyLightLutBufferSRV[0].rgb;
#else
	// Use a structured buffer for scalar loads.
	return View.DistantSkyLightLutBufferSRV[0].rgb; 
#endif
}