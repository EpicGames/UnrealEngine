// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyLightingDiffuseShared.usf
=============================================================================*/

#pragma once

#include "SkyLightingShared.ush"
#include "ClearCoatCommon.ush"
#include "ShadingEnergyConservation.ush"
#include "HairBsdf.ush"
#include "HairStrands/HairStrandsEnvironmentLightingCommon.ush"

#ifndef APPLY_SKY_SHADOWING
#define APPLY_SKY_SHADOWING 0
#endif

float3 ContrastAndNormalizeMulAdd;
float OcclusionExponent;
float OcclusionCombineMode;
// The following paramaters are declared in SkyLightingShared.usf
// float4 OcclusionTintAndMinOcclusion;

struct FSkyLightVisibilityData
{
	float  SkyDiffuseLookUpMul;
	float3 SkyDiffuseLookUpAdd;
	float3 SkyDiffuseLookUpNormal;
};

FSkyLightVisibilityData GetSkyLightVisibilityData(float3 SkyLightingNormal, const float3 WorldNormal, const float GBufferAO, float ScreenAO, const float3 BentNormal)
{
	float SkyVisibility = 1;
	float DotProductFactor = 1;

#if APPLY_SKY_SHADOWING
	#define USE_DIRECTIONAL_OCCLUSION_ON_SKY_DIFFUSE 1
	#if USE_DIRECTIONAL_OCCLUSION_ON_SKY_DIFFUSE
	{
		SkyVisibility = length(BentNormal);
		float3 NormalizedBentNormal = BentNormal / (max(SkyVisibility, .00001f));

		// Use more bent normal in corners
		float BentNormalWeightFactor = SkyVisibility;

		SkyLightingNormal = lerp(NormalizedBentNormal, WorldNormal, BentNormalWeightFactor);
		DotProductFactor = lerp(dot(NormalizedBentNormal, WorldNormal), 1, BentNormalWeightFactor);
	}
	#else
	{
		SkyVisibility = length(BentNormal);
	}
	#endif

	float ContrastCurve = 1 / (1 + exp(-ContrastAndNormalizeMulAdd.x * (SkyVisibility * 10 - 5)));
	SkyVisibility = saturate(ContrastCurve * ContrastAndNormalizeMulAdd.y + ContrastAndNormalizeMulAdd.z);

	// Apply DFAO controls
	SkyVisibility = pow(SkyVisibility, OcclusionExponent);
	SkyVisibility = lerp(SkyVisibility, 1, OcclusionTintAndMinOcclusion.w);
#endif

	// Combine with other AO sources
	if (OcclusionCombineMode == 0)
	{
		// Combine with min which nicely avoids over-occlusion in cases where strong DFAO is present along with strong SSAO (distant trees)
		SkyVisibility = min(SkyVisibility, min(GBufferAO, ScreenAO));
	}
	else
	{
		// Combine with mul, which continues to add SSAO depth even indoors.  SSAO will need to be tweaked to be less strong.
		SkyVisibility = SkyVisibility * min(GBufferAO, ScreenAO);
	}

	// Apply AO to the sky diffuse and account for darkening due to the geometry term
	// apply the Diffuse color to the lighting (including OcclusionTintAndMinOcclusion as it's considered another light, that fixes SubsurfaceProfile being too dark)
	FSkyLightVisibilityData SkyVisData;
	SkyVisData.SkyDiffuseLookUpMul = SkyVisibility * DotProductFactor;
	SkyVisData.SkyDiffuseLookUpAdd = (1 - SkyVisibility) * OcclusionTintAndMinOcclusion.xyz;
	SkyVisData.SkyDiffuseLookUpNormal = SkyLightingNormal;
	return SkyVisData;
}

float3 SkyLightDiffuse(FGBufferData GBuffer, float AmbientOcclusion, float2 BufferUV, float2 ScreenPosition, float3 BentNormal, float3 DiffuseColor)
{
	float2 UV = BufferUV;
	float3 Lighting = 0;

	float3 SkyLightingNormal = GetClearCoatBottomNormal(GBuffer, GBuffer.WorldNormal);
	FSkyLightVisibilityData SkyVisData = GetSkyLightVisibilityData(SkyLightingNormal, GBuffer.WorldNormal, GBuffer.GBufferAO, AmbientOcclusion, BentNormal);

	const float3 N = GBuffer.WorldNormal;
	const float3 V = -normalize(mul(float3(ScreenPosition, 1), DFToFloat3x3(PrimaryView.ScreenToWorld)).xyz);
	const float NoV = saturate(dot(V, N));
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(GBuffer.Roughness, NoV, GBuffer.SpecularColor);
	float3 DiffuseWeight = ComputeEnergyPreservation(EnergyTerms);

	BRANCH
	if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		float3 SubsurfaceLookup = GetSkySHDiffuse(-GBuffer.WorldNormal) * View.SkyLightColor.rgb;
		float3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
		Lighting += SkyVisData.SkyDiffuseLookUpMul * SubsurfaceLookup * SubsurfaceColor;
	}

	if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		float3 SubsurfaceColor = ExtractSubsurfaceColor(GBuffer);
		// Add subsurface energy to diffuse
		DiffuseColor += SubsurfaceColor;
	}

	BRANCH
	if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
	{
		float3 L = 0;
		DiffuseColor = EvaluateEnvHair(GBuffer, V, N, L /*out*/);
		DiffuseWeight = 1.0f;
		SkyVisData.SkyDiffuseLookUpNormal = L;
	}

	if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
	{
		float3 ClothFuzz = ExtractSubsurfaceColor(GBuffer);
		DiffuseColor += ClothFuzz * GBuffer.CustomData.a;
	}

	// Compute the preconvolved incoming lighting with the bent normal direction
	float3 DiffuseLookup = GetSkySHDiffuse(SkyVisData.SkyDiffuseLookUpNormal) * View.SkyLightColor.rgb;

	// And accumulate the lighting
	Lighting += (SkyVisData.SkyDiffuseLookUpMul * DiffuseLookup + SkyVisData.SkyDiffuseLookUpAdd) * DiffuseColor * DiffuseWeight;

#if FEATURE_LEVEL > FEATURE_LEVEL_ES3_1
	Lighting *= View.PreExposure;
#endif

	//Lighting = (Texture2DSampleLevel(BentNormalAOTexture, BentNormalAOSampler, UV, 0).xyz);

	return Lighting;
}
