// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SkyLightingShared.usf
=============================================================================*/

#pragma once

float ApproximateConeConeIntersection(float ArcLength0, float ArcLength1, float AngleBetweenCones)
{
	float AngleDifference = abs(ArcLength0 - ArcLength1);

	float Intersection = smoothstep(
		0,
		1.0,
		1.0 - saturate((AngleBetweenCones - AngleDifference) / (ArcLength0 + ArcLength1 - AngleDifference)));

	return Intersection;
}

float ApplyBentNormalAO;
float InvSkySpecularOcclusionStrength;
float4 OcclusionTintAndMinOcclusion;

void GetDistanceFieldAOSpecularOcclusion(float3 BentNormalAO, float3 ReflectionVector, float Roughness, bool bTwoSidedFoliage, out float IndirectSpecularOcclusion, out float IndirectDiffuseOcclusion, out float3 ExtraIndirectSpecular)
{
	IndirectSpecularOcclusion = 1;
	IndirectDiffuseOcclusion = 1;
	ExtraIndirectSpecular = 0;

	BRANCH
	if (ApplyBentNormalAO > 0)
	{
		float BentNormalLength = length(BentNormalAO);

		BRANCH
		if (View.DistanceFieldAOSpecularOcclusionMode == 0)
		{
			IndirectSpecularOcclusion = BentNormalLength;
		}
		else
		{
			BRANCH
			if (bTwoSidedFoliage)
			{
				IndirectSpecularOcclusion = BentNormalLength;
			}
			else
			{
				float ReflectionConeAngle = max(Roughness, .1f) * PI;
				float UnoccludedAngle = BentNormalLength * PI * InvSkySpecularOcclusionStrength;
				float AngleBetween = acos(dot(BentNormalAO, ReflectionVector) / max(BentNormalLength, .001f));
				IndirectSpecularOcclusion = ApproximateConeConeIntersection(ReflectionConeAngle, UnoccludedAngle, AngleBetween);

				// Can't rely on the direction of the bent normal when close to fully occluded, lerp to shadowed
				IndirectSpecularOcclusion = lerp(0, IndirectSpecularOcclusion, saturate((UnoccludedAngle - .1f) / .2f));
			}
		}

		IndirectSpecularOcclusion = lerp(IndirectSpecularOcclusion, 1, OcclusionTintAndMinOcclusion.w);
		ExtraIndirectSpecular = (1 - IndirectSpecularOcclusion) * OcclusionTintAndMinOcclusion.xyz;
		IndirectDiffuseOcclusion = lerp(BentNormalLength, 1, OcclusionTintAndMinOcclusion.w);
	}
}

float GetDynamicSkyIndirectIrradiance(float3 BentNormal, float3 WorldNormal)
{
	float SkyVisibility = length(BentNormal);
	float3 DiffuseLookup = GetSkySHDiffuse(WorldNormal) * View.SkyLightColor.rgb;
	return Luminance(DiffuseLookup) * SkyVisibility;
}