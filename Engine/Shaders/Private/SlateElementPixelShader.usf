// Copyright Epic Games, Inc. All Rights Reserved.


#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "SlateShaderCommon.ush"

#pragma message("UESHADERMETADATA_VERSION F34E9FD2-616B-4620-95C5-C123D78CE6AA")

#ifndef SHADER_TYPE
#define SHADER_TYPE 0
#endif

#ifndef SAMPLE_VIRTUAL_TEXTURE
#define SAMPLE_VIRTUAL_TEXTURE 0
#endif

#if SAMPLE_VIRTUAL_TEXTURE
	#define NUM_VIRTUALTEXTURE_SAMPLES 1
	// Workaround for unbound View uniform buffer
	#define VT_DISABLE_VIEW_UNIFORM_BUFFER 1
	#include "VirtualTextureCommon.ush"
#endif

/** Display gamma x:gamma curve adjustment, y:inverse gamma (1/GEngine->DisplayGamma), z:InvertAlpha, w:Contrast */
half4 GammaAndAlphaValues;

/** Per element params */
float4 ShaderParams;
float4 ShaderParams2;

#define USE_LEGACY_DISABLED_EFFECT 0

/** x: DrawDisabledEffect */
/** y: ShaderType*/
uint4 DrawFlags;

uint GetSlateShaderType()
{
#if SHADER_TYPE == ST_Dynamic
	return DrawFlags.y;
#else
	return SHADER_TYPE;
#endif
}

/** Texture for the element */
Texture2D ElementTexture;
SamplerState ElementTextureSampler;

struct FontSignedDistanceSample
{
	MaterialFloat DistanceSample;
	MaterialFloat SmoothDistanceSample;
	float ScreenSpaceSpread;
};

struct FMaterialColorParameters
{
	half4 MaterialColor;
	
	float4 SdfData;
	float2 MeshUVs;
	
	FontSignedDistanceSample SdfSample;
	half4 FontSampledColor;

	bool bNeedMaterialColor;
};

/**
 * Applys a contrast about a midpoint squeezing the colors towards dark or light depending on
 * the midpoint.  Additionally - it increases the opaqueness of colors to try and seperate them from
 * the background.
 */
half4 ApplyContrast(half4 InColor)
{
    FLATTEN if( GammaAndAlphaValues.w != 1.0f )
    {
		float ContrastMidpoint = 0.25; //TODO We may want to make the midpoint configurable so that users can pick a different one, based on the overall saturation of their UI.
	    InColor.rgb = saturate(((InColor.rgb - ContrastMidpoint) * GammaAndAlphaValues.w) + ContrastMidpoint);
    }
    
    return InColor;
}

half3 GammaCorrect(half3 InColor)
{
	half3 CorrectedColor = InColor;

#if SOURCE_IN_LINEAR_SPACE
	FLATTEN if( GammaAndAlphaValues.y != 1.0f )
	{
		CorrectedColor = ApplyGammaCorrection(CorrectedColor, GammaAndAlphaValues.x);
	}
#endif

	return CorrectedColor;
}

void GetSlateUVs(VertexToPixelInterpolants Interpolants, inout float2 UVs[8])
{
#if NUM_MATERIAL_TEXCOORDS > 0
	UVs[0] = SLATE_UV0( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 1
	UVs[1] = SLATE_UV1( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 2
	UVs[2] = SLATE_UV2( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 3
	UVs[3] = SLATE_UV3( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 4
	UVs[4] = SLATE_UV4( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 5
	UVs[5] = SLATE_UV5( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 6
	UVs[6] = SLATE_UV6( Interpolants );
#endif
#if NUM_MATERIAL_TEXCOORDS > 7
	UVs[7] = SLATE_UV7( Interpolants );
#endif
}

#if USE_MATERIALS
void FillTexCoords( inout FMaterialPixelParameters Parameters, VertexToPixelInterpolants InVertex, float2 InDefaultUV )
{
	const uint ShaderType = GetSlateShaderType();

	if (ShaderType == ST_GrayscaleFont || ShaderType == ST_ColorFont || ShaderType == ST_SdfFont || ShaderType == ST_MsdfFont)
	{
		#if NUM_MATERIAL_TEXCOORDS == 1
			// Note: The first channel is used to look up into the texture atlas. The second channel is the custom 0-1 UV set for the text run
			Parameters.TexCoords[0] = SLATE_UV1( InVertex );
		#elif NUM_MATERIAL_TEXCOORDS == 2
			// Note: The first channel is used to look up into the texture atlas. The second channel is the custom 0-1 UV set for the text run
			Parameters.TexCoords[0] = SLATE_UV1( InVertex );
			Parameters.TexCoords[1] = InVertex.MaterialTexCoords;
		#endif

		return;
	}

#if NUM_MATERIAL_TEXCOORDS > 0
	if (ShaderType == ST_Custom || NUM_CUSTOMIZED_UVS != 0)
	{
		// note: We assume users who customize UVs or are using completely custom elements know what they are doing with UVs and want no assumptions made about how uvs are used
		float2 UVs[8]; GetSlateUVs(InVertex, UVs);
		UNROLL
		for (int CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
		{
			Parameters.TexCoords[CoordinateIndex] = UVs[CoordinateIndex];
		}

		return;
	}
#endif // NUM_MATERIAL_TEXCOORDS > 0

#if NUM_MATERIAL_TEXCOORDS == 1
	Parameters.TexCoords[0] = InDefaultUV;
#elif NUM_MATERIAL_TEXCOORDS == 2
	Parameters.TexCoords[0] = InDefaultUV;
	Parameters.TexCoords[1] = SLATE_UV0(InVertex);
#elif NUM_MATERIAL_TEXCOORDS == 3
	Parameters.TexCoords[0] = InDefaultUV;
	Parameters.TexCoords[1] = SLATE_UV0(InVertex);
	Parameters.TexCoords[2] = SLATE_UV1(InVertex);
#elif NUM_MATERIAL_TEXCOORDS >= 4
	Parameters.TexCoords[0] = InDefaultUV;
	Parameters.TexCoords[1] = SLATE_UV0(InVertex);
	Parameters.TexCoords[2] = SLATE_UV1(InVertex);

	float2 UVs[8]; GetSlateUVs(InVertex, UVs);
	UNROLL
	for (int CoordinateIndex = 3; CoordinateIndex < NUM_MATERIAL_TEXCOORDS; CoordinateIndex++)
	{
		Parameters.TexCoords[CoordinateIndex] = UVs[CoordinateIndex];
	}
#endif // NUM_MATERIAL_TEXCOORDS == 1
}

half4 GetMaterialColor( VertexToPixelInterpolants InVertex, float2 MeshUV, float4 InFontSignedDistanceData = float4(0.0, 0.0, 0.0, 0.0) )
{
	ResolvedView = ResolveView();
	FMaterialPixelParameters Parameters = MakeInitializedMaterialPixelParameters();

	// Inputs might require texcoords, so fill them in first
	FillTexCoords( Parameters, InVertex, MeshUV );

	Parameters.VertexColor = InVertex.Color;
#if SHADER_TYPE == ST_GrayscaleFont || SHADER_TYPE == ST_ColorFont || SHADER_TYPE == ST_SdfFont || SHADER_TYPE == ST_MsdfFont || SHADER_TYPE == ST_Dynamic
	Parameters.FontSignedDistanceData = InFontSignedDistanceData;
#endif

	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters( Parameters, PixelMaterialInputs, InVertex.Position, true );

	// Clip if masked
	GetMaterialCoverageAndClipping( Parameters, PixelMaterialInputs );

	half4 OutColor;
	
#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE

	// UI node forces a single unlit BSDF. We get the raw data from the Substrate UI node from the raw BSDF on the tree.
	FSubstratePixelHeader SubstratePixelHeader = Parameters.GetFrontSubstrateHeader();
	FSubstrateBSDF UnlitBSDF = SubstratePixelHeader.SubstrateTree.BSDFs[0];
	UnlitBSDF.SubstrateSanitizeBSDF(); // required wehen not calling SubstrateUpdateTreeUnlit

	float3 UnlitLuminance = BSDF_GETEMISSIVE(UnlitBSDF);
	float3 UnlitTransmittance= UNLIT_TRANSMITTANCE(UnlitBSDF);
	float UnlitGreyCoverage = 1.0 - UnlitTransmittance.x;

#if ((SUBSTRATE_OPAQUE_DEFERRED || (SUBSTRATE_OPAQUE_MATERIAL && SUBSTRATE_FORWARD_SHADING)))

	OutColor = half4(UnlitLuminance, 1.0f);

#else // SUBSTRATE_OPAQUE_DEFERRED

#if MATERIALBLENDING_ALPHACOMPOSITE

#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
	UnlitGreyCoverage = GetMaterialOpacity(PixelMaterialInputs);
#endif
	OutColor = half4(UnlitLuminance, UnlitGreyCoverage);

#elif MATERIALBLENDING_MODULATE
	OutColor = half4(UnlitTransmittance, 0.0f);
#elif MATERIALBLENDING_ALPHAHOLDOUT
	OutColor = half4(0.0f, 0.0f, 0.0f, UnlitGreyCoverage);
#elif MATERIALBLENDING_ADDITIVE
	OutColor = half4(UnlitLuminance, 1.0);
#else // MATERIALBLENDING_TRANSLUCENT
	OutColor = half4(UnlitLuminance, UnlitBSDF.Coverage);
#endif // MATERIALBLENDING_xxx

#endif// SUBSTRATE_OPAQUE_DEFERRED

#else // SUBSTRATE_ENABLED

	half Opacity = GetMaterialOpacity(PixelMaterialInputs);
	half3 Color = GetMaterialEmissive(PixelMaterialInputs);

#if MATERIALBLENDING_ADDITIVE
	OutColor = half4(Color * Opacity, 0.0f);
#else
	OutColor = half4(Color, Opacity);
#endif

#endif // SUBSTRATE_ENABLED

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(Parameters.VirtualTextureFeedback, Parameters.SvPosition, View.VTFeedbackBuffer);
#endif

	return OutColor;
}
#endif

#if !USE_MATERIALS
half4 GetTextureColor( VertexToPixelInterpolants InVertex, float2 UV )
{
	half4 BaseColor;

	#if SAMPLE_VIRTUAL_TEXTURE
		FVirtualTextureFeedbackParams VirtualTextureFeedback;
		InitializeVirtualTextureFeedback(VirtualTextureFeedback);
		VTPageTableUniform PageTableUniform = VTPageTableUniform_Unpack(SlateElementVirtualTextureParams.PackedPageTableUniform0, SlateElementVirtualTextureParams.PackedPageTableUniform1);
		VTPageTableResult PageTableResult = TextureLoadVirtualPageTable(SlateElementVirtualTextureParams.PageTableTexture, PageTableUniform, UV, VTADDRESSMODE_WRAP, VTADDRESSMODE_WRAP, 0.0f, InVertex.Position.xy, VirtualTextureFeedback); 
		BaseColor = TextureVirtualSampleLevel(ElementTexture, ElementTextureSampler, PageTableResult, SlateElementVirtualTextureParams.LayerIndex, VTUniform_Unpack(SlateElementVirtualTextureParams.PackedUniform));
		FinalizeVirtualTextureFeedback(VirtualTextureFeedback, SlateElementVirtualTextureParams.FeedbackParams, InVertex.Position, SlateElementVirtualTextureParams.FeedbackBuffer);
	#else
		BaseColor = Texture2DSample(ElementTexture, ElementTextureSampler, UV );
	#endif

	#if USE_TEXTURE_GRAYSCALE
		#if !USE_TEXTURE_ALPHA
			BaseColor.rgb = BaseColor.rrr;
			BaseColor.a = 1;
		#else
			BaseColor.a = BaseColor.r;
			BaseColor.rgb = half3(1,1,1);
		#endif
	#else
		#if !USE_TEXTURE_ALPHA
			BaseColor.a = 1.0f;
		#endif
	#endif

	return BaseColor;
}
#endif

/** Gets a color from a texture and supplied vertex color.  Adjusting for tint masking if necessary */
half4 GetColor( VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters)
{
	half4 FinalColor;
	
#if USE_MATERIALS
	half4 BaseColor = MaterialColorParameters.MaterialColor;
#else
	half4 BaseColor = GetTextureColor( InVertex, MaterialColorParameters.MeshUVs );
#endif

	const uint ShaderType = GetSlateShaderType();

	if (ShaderType == ST_Custom)
	{
		// Make no assumptionn of what vertex color does in a custom material shader
		FinalColor = BaseColor;	
	}
	else
	{
		FinalColor = BaseColor * InVertex.Color;
#if USE_MATERIALS && MATERIALBLENDING_ADDITIVE
		FinalColor *= InVertex.Color.a;
#endif
	}

	return FinalColor;
}

FontSignedDistanceSample SampleFontSignedDistance(float2 TextureCoordinate)
{
	FontSignedDistanceSample OutSample;
	const MaterialFloat4 MultiDistance = Texture2DSample(ElementTexture, ElementTextureSampler, TextureCoordinate);

	const uint ShaderType = GetSlateShaderType();

	if (GetSlateShaderType() == ST_MsdfFont)
	{
		OutSample.DistanceSample = max(min(MultiDistance.r, MultiDistance.g), min(max(MultiDistance.r, MultiDistance.g), MultiDistance.b));
		OutSample.SmoothDistanceSample = MultiDistance.a;
	}
	else
	{
		OutSample.DistanceSample = lerp(MultiDistance A8_SAMPLE_MASK, MultiDistance.a, ShaderParams2.x);
		OutSample.SmoothDistanceSample = OutSample.DistanceSample;
	}

	const float2 ScreenSpaceTextureSize = float2(1.0, 1.0) / fwidth(TextureCoordinate);
	OutSample.ScreenSpaceSpread = max(dot(ShaderParams.xy, ScreenSpaceTextureSize), 1.0);
	return OutSample;
}

float GetFontSignedDistanceOpacity(FontSignedDistanceSample InSample)
{
	const float ScreenSpaceDistance = (InSample.DistanceSample - ShaderParams.z) * InSample.ScreenSpaceSpread;
	return saturate(ScreenSpaceDistance + 0.5);
}

#if USE_MATERIALS
float4 GetFontSignedDistanceData(FontSignedDistanceSample InSample)
{
	const float EmSignedDistance = (InSample.DistanceSample - ShaderParams.z) * ShaderParams.w;
	const float EmSmoothDistance = (InSample.SmoothDistanceSample - ShaderParams.z) * ShaderParams.w;
	const float EmPxFactor = InSample.ScreenSpaceSpread / ShaderParams.w;
	return float4(EmSignedDistance, EmSmoothDistance, EmPxFactor, GetFontSignedDistanceOpacity(InSample));
}
#endif

half4 GetSdfTextElementColor(VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters)
{
	const FontSignedDistanceSample SdfSample = MaterialColorParameters.SdfSample;
	half4 OutColor = InVertex.Color;
#if USE_MATERIALS
	const float4 SdfData = MaterialColorParameters.SdfData;
	OutColor *= MaterialColorParameters.MaterialColor;
	if (ShaderParams2.y < 0.5) // ShaderParams2.y is bMaterialIsStencil
	{
	#if MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_ALPHACOMPOSITE
		OutColor *= SdfData.a;
	#else
		OutColor.a *= SdfData.a;
	#endif
	}
#else // !USE_MATERIALS
	OutColor.a *= GetFontSignedDistanceOpacity(SdfSample);
#endif
	return OutColor;
}

MaterialFloat4 GetFontElementSampledColor(VertexToPixelInterpolants InVertex)
{
	float2 AtlasTextureCoordinates = SLATE_UV0( InVertex );
	MaterialFloat4 SampledColor = MaterialFloat4(1,1,1,1);

	if (GetSlateShaderType() == ST_GrayscaleFont)
	{
#if USE_MATERIALS && (MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_ALPHACOMPOSITE)
		SampledColor *= Texture2DSample_A8(ElementTexture, ElementTextureSampler, AtlasTextureCoordinates);
#else
		SampledColor.a = Texture2DSample_A8(ElementTexture, ElementTextureSampler, AtlasTextureCoordinates);
#endif
	}
	else
	{
		SampledColor = Texture2DSample(ElementTexture, ElementTextureSampler, AtlasTextureCoordinates);
	}

	return SampledColor;
}

half4 GetFontElementColor( VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters )
{
	MaterialFloat4 SampledColor = MaterialColorParameters.FontSampledColor;

	half4 OutColor = InVertex.Color;
#if USE_MATERIALS
	OutColor *= MaterialColorParameters.MaterialColor;
	if (ShaderParams2.y < 0.5) // ShaderParams2.y is bMaterialIsStencil
	{
		OutColor *= half4(SampledColor);
	}
#else
	OutColor *= half4(SampledColor);
#endif

	return OutColor;
}

half4 GetDefaultElementColor( VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters )
{
	return GetColor( InVertex, MaterialColorParameters );
}

float2 GetBorderElementColorUV(VertexToPixelInterpolants InVertex)
{
	float4 TextureCoordinates = InVertex.TextureCoordinates0;
	float2 NewUV;
	if( TextureCoordinates.z == 0.0f && TextureCoordinates.w == 0.0f )
	{
		NewUV = TextureCoordinates.xy;
	}
	else
	{
		float2 MinUV;
		float2 MaxUV;
	
		if( TextureCoordinates.z > 0 )
		{
			MinUV = float2(ShaderParams.x,0);
			MaxUV = float2(ShaderParams.y,1);
			TextureCoordinates.w = 1.0f;
		}
		else
		{
			MinUV = float2(0,ShaderParams.z);
			MaxUV = float2(1,ShaderParams.w);
			TextureCoordinates.z = 1.0f;
		}

		NewUV = TextureCoordinates.xy*TextureCoordinates.zw;
		NewUV = frac(NewUV);
		NewUV = lerp(MinUV,MaxUV,NewUV);	
	}

	return NewUV;
}

half4 GetBorderElementColor( VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters )
{
	return GetColor( InVertex, MaterialColorParameters );
}

half4 GetRoundedBoxElementColor( VertexToPixelInterpolants InVertex, FMaterialColorParameters MaterialColorParameters )
{
	return GetRoundedBoxElementColorInternal(ShaderParams.zw, InVertex.MaterialTexCoords.xy, ShaderParams.y, ShaderParams2, GetColor(InVertex, MaterialColorParameters),  InVertex.SecondaryColor);
}

/**
 * Computes 0-1 value that interpolates between dashes of length DashLength based on Distance with 1px of antialiasing between dashes and gaps
 * 
 * ie:
 * 1 -----------------                     -----------------                     -----------------                     
 *                    \                   /                 \                   /                 \                   /
 * 0                   \_________________/                   \_________________/                   \_________________/ 
 *
 *   <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >
 
 */
float ModulateDashedLine(float Distance, float DashLength)
{
	// Add one for antialias pixel
	DashLength += 1.f;

	const float HalfDash  = 0.5f*DashLength;
	const float Period    = 2.0f*DashLength;

	// Offset the interpolation so that the dash starts at distance = 0;
	const float Interp = Distance - HalfDash + 0.5f;
	const float Mod = Interp - Period*floor(Interp/Period); // mod(Interp, Period)

	// Compute the dash alpha using a clamped triangle wave
	const float TriWave = 2.0f * abs(Mod - DashLength) - DashLength;
	return clamp(TriWave, -1.0f, 1.0f)*0.5f + 0.5f;
}

/**
 * Generates an anti-aliased line segment pixel
 * This is based on the fast prefiltered lines technique published in GPU Gems 2
 *
 * Instead of passing edge function coefficients as uniform parameters we use
 * texture coordinates to define the line
 *
 * When rasterized, the submitted geometry must contain every pixel that
 * could have coverage > 0. Failing to do so will create sharp, aliased edges
 */
half4 GetLineSegmentElementColor( VertexToPixelInterpolants InVertex )
{
	const float2 Gradient = SLATE_UV0(InVertex);
	const float2 DashParams = SLATE_UV1(InVertex);

	// Get coverage based on distance from segment sides and ends
	const float2 OutsideFilterUV = float2(1.0f, 1.0f);
	const float2 InsideFilterUV = float2(ShaderParams.x, 0.0f);
	const float2 LineCoverage = smoothstep(OutsideFilterUV, InsideFilterUV, abs(Gradient));

	const float DashLength = DashParams.y;
	const float DashAlpha = ModulateDashedLine(DashParams.x, DashParams.y);

	half4 Color = InVertex.Color;
	Color.a *= LineCoverage.x * LineCoverage.y * DashAlpha;
	return Color;
}

FMaterialColorParameters GetMaterialColorParameters(VertexToPixelInterpolants InVertex, const uint ShaderType)
{
	FMaterialColorParameters Output = (FMaterialColorParameters)0;
	Output.bNeedMaterialColor = true;

	if (ShaderType == ST_Default || ShaderType == ST_Custom)
	{
		Output.MeshUVs = SLATE_UV0(InVertex) * SLATE_UV1(InVertex);
	}
	else if (ShaderType == ST_RoundedBox)
	{
		Output.MeshUVs = SLATE_UV0(InVertex) * SLATE_UV1(InVertex);
	}
	else if (ShaderType == ST_Border)
	{
		Output.MeshUVs = GetBorderElementColorUV(InVertex);
	}
	else if (ShaderType == ST_GrayscaleFont || ShaderType == ST_ColorFont)
	{
		Output.MeshUVs = float2(0,0);
		Output.FontSampledColor = GetFontElementSampledColor(InVertex);
		Output.SdfData = float4(0, 0, 0, Output.FontSampledColor.a);
	}
	else if (ShaderType == ST_SdfFont || ShaderType == ST_MsdfFont)
	{
		Output.MeshUVs = float2(0, 0);
		Output.SdfSample = SampleFontSignedDistance(SLATE_UV0(InVertex));
#if USE_MATERIALS
		Output.SdfData = GetFontSignedDistanceData(Output.SdfSample);
#endif
	}
	else
	{
		Output.bNeedMaterialColor = false;
		// Not using Material Color
		Output.MeshUVs = float2(0, 0);
	}

	return Output;
}

HALF4_TYPE Main(VertexToPixelInterpolants VIn) : SV_Target0
{
	half4 OutColor;
	const uint ShaderType = GetSlateShaderType();

	FMaterialColorParameters MaterialColorParameters = GetMaterialColorParameters(VIn, ShaderType);

	BRANCH
	if (MaterialColorParameters.bNeedMaterialColor)
	{
#if USE_MATERIALS
		MaterialColorParameters.MaterialColor = GetMaterialColor(VIn, MaterialColorParameters.MeshUVs, MaterialColorParameters.SdfData);
#endif
	}

	if (ShaderType == ST_Default || ShaderType == ST_Custom)
	{
		OutColor = GetDefaultElementColor(VIn, MaterialColorParameters);
	}
	else if (ShaderType == ST_RoundedBox)
	{
		OutColor = GetRoundedBoxElementColor( VIn, MaterialColorParameters );
	}
	else if (ShaderType == ST_Border)
	{
		OutColor = GetBorderElementColor(VIn, MaterialColorParameters);
	}
	else if (ShaderType == ST_GrayscaleFont || ShaderType == ST_ColorFont)
	{
		OutColor = GetFontElementColor(VIn, MaterialColorParameters);
	}
	else if (ShaderType == ST_SdfFont || ShaderType == ST_MsdfFont)
	{
		OutColor = GetSdfTextElementColor(VIn, MaterialColorParameters);
	}
	else
	{
		OutColor = GetLineSegmentElementColor(VIn);
	}

	// For standard slate which only has a few shaders, the permutation is fine here,
	// for Engine Materials, we don't ever define this macro, so instead we fallback
	// to looking at the shader constants.
#if USE_MATERIALS
	const half DrawDisabledEffect = DrawFlags.x;
#else
	#if DRAW_DISABLED_EFFECT
		const half DrawDisabledEffect = 1;
	#else
		const half DrawDisabledEffect = 0;
	#endif
#endif

	if (DrawDisabledEffect != 0)
	{
#if USE_LEGACY_DISABLED_EFFECT
		#if SOURCE_IN_LINEAR_SPACE
			half3 LumCoeffs = half3( 0.3, 0.59, .11 );
			half3 Grayish = half3(.1, .1, .1);
		#else
			// These are just the linear space values above converted to sRGB space.
			half3 LumCoeffs = float3( 0.5843, 0.7921, 0.3647 );
			half3 Grayish = float3(0.349, 0.349, 0.349);
		#endif
    
		//desaturate
		half Lum = dot( LumCoeffs, OutColor.rgb );
		OutColor.rgb = lerp( OutColor.rgb, float3(Lum,Lum,Lum), .8 );
		OutColor.rgb = lerp( OutColor.rgb, Grayish, clamp( distance( OutColor.rgb, Grayish ), 0, .8)  );
#else
		//float bw = (min(OutColor.r, min(OutColor.g, OutColor.b)) + max(OutColor.r, max(OutColor.g, OutColor.b))) * 0.5f;
		//OutColor.rgb = lerp(OutColor.rgb, float3(bw, bw, bw), .5);

		OutColor.a *= .45f;
#endif
	}

	// Contrast
	OutColor = ApplyContrast(OutColor);

	// Gamma Correct
	OutColor.rgb = GammaCorrect(OutColor.rgb);

#if !USE_MATERIALS
    // Deal with cases where the incoming alpha is inverted like if you're dealing with render targets coming
    // from the scene renderer.
    OutColor.a = lerp(OutColor.a, 1 - OutColor.a, GammaAndAlphaValues.z);
#endif
    
	return OutColor;
}

float4 DebugOverdrawMain( VertexToPixelInterpolants VIn ) : SV_Target0
{
	return .1;
}

/** Batch Color */
float4 BatchColor;

float4 DebugBatchingMain(VertexToPixelInterpolants VIn) : SV_Target0
{
    return BatchColor;
}

