// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

float4 MaskRectPacked[2];

struct VertexToPixelInterpolants
{
    float4 Position : SV_POSITION;
};

VertexToPixelInterpolants MainVS(in uint InPosition : ATTRIBUTE0)
{
	VertexToPixelInterpolants VOut = (VertexToPixelInterpolants)0;
	const float2 MaskRect[4] = { MaskRectPacked[0].xy, MaskRectPacked[0].zw, MaskRectPacked[1].xy, MaskRectPacked[1].zw };
	VOut.Position = float4(MaskRect[InPosition], 0, 1);
	return VOut;
}

float4 MainPS(VertexToPixelInterpolants VIn) : SV_Target0
{
	return 1;
}