// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SlateShaderCommon.ush"
#include "GammaCorrectionCommon.ush"
#include "ColorSpace.ush"
#include "ColorDeficiency.ush"

Texture2D ElementTexture;
SamplerState ElementTextureSampler;

float ColorVisionDeficiencyType;
float ColorVisionDeficiencySeverity;
float bCorrectDeficiency;
float bSimulateCorrectionWithDeficiency;

float4 ColorDeficiencyMain(noperspective float4 UVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput) : SV_Target0
{
	float4 SourceColor = Texture2DSample(ElementTexture, ElementTextureSampler, UVAndScreenPos.xy);
	float3 OutRGB = ColorDeficiency(SourceColor.rgb, ColorVisionDeficiencyType, ColorVisionDeficiencySeverity, bCorrectDeficiency, bSimulateCorrectionWithDeficiency);
	return float4(OutRGB, SourceColor.a);
}