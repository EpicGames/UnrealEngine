// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SlateShaderCommon.ush"
#include "GammaCorrectionCommon.ush"

#define MAX_SAMPLES 127

// Weigts and offsets are packed into 4 floats (Weight, Offset, Weight, Offset)
float4 WeightAndOffsets[MAX_SAMPLES/2];

/** Blur sample count */
int SampleCount;

Texture2D ElementTexture;
SamplerState ElementTextureSampler;

float4 BufferSizeAndDirection;
float4 UVBounds;
float4 ShaderParams;
float4 ShaderParams2;

float4 GetSample(float Weight, float Offset, float2 UV)
{
	const float2 MinUV = UVBounds.xy;
	const float2 MaxUV = UVBounds.zw;
	const float2 Direction = BufferSizeAndDirection.zw;
	const float2 BufferSize = BufferSizeAndDirection.xy;
	const float2 UVOffset = float2(Offset*BufferSize.x*Direction.x, Offset*BufferSize.y*Direction.y);

	return 
		Texture2DSample(ElementTexture, ElementTextureSampler, clamp(UV + UVOffset, MinUV, MaxUV)) * Weight
		+ Texture2DSample(ElementTexture, ElementTextureSampler, clamp(UV - UVOffset, MinUV, MaxUV)) * Weight;
}

float4 GaussianBlurMain(in noperspective float4 InUVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput) : SV_Target0
{
	float4 OutColor = Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy, UVBounds.xy, UVBounds.zw)) * WeightAndOffsets[0].x;
	// First offset is in zw
	{
		float Weight = WeightAndOffsets[0].z;
		float Offset = WeightAndOffsets[0].w;

		OutColor += GetSample(Weight, Offset, InUVAndScreenPos.xy);
	}

	for (int i = 2; i<SampleCount; i+=2)
	{
		int Index = i/2;
		{
			float Weight = WeightAndOffsets[Index].x;
			float Offset = WeightAndOffsets[Index].y;

			OutColor += GetSample(Weight, Offset, InUVAndScreenPos.xy);
		}

		{
			float Weight = WeightAndOffsets[Index].z;
			float Offset = WeightAndOffsets[Index].w;

			OutColor += GetSample(Weight, Offset, InUVAndScreenPos.xy);
		}

	}

	return float4(OutColor.rgb, 1);
}

float4 OptimizedKawaseUpsampleMain(in noperspective float4 InUVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput, in float4 SvPosition : SV_POSITION) : SV_Target0
{
	float2 OffsetScale = 4.0 * (0.5 - frac(0.5 * SvPosition.xy)) * ShaderParams.xy;
	return (
		ShaderParams.z * Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy + ShaderParams2.z * OffsetScale, UVBounds.xy, UVBounds.zw)) +
		0.5 * (1.0 - (ShaderParams.z + ShaderParams.w)) * (
			Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy + ShaderParams2.xy * OffsetScale, UVBounds.xy, UVBounds.zw)) +
			Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy + ShaderParams2.yx * OffsetScale, UVBounds.xy, UVBounds.zw))
		) +
		ShaderParams.w * Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy + ShaderParams2.w * OffsetScale , UVBounds.xy, UVBounds.zw))
	);
}

float4 Resample1Main(in noperspective float4 InUVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput) : SV_Target0
{
	return Texture2DSample(ElementTexture, ElementTextureSampler, clamp(InUVAndScreenPos.xy, UVBounds.xy, UVBounds.zw));
}

// Used both for downscaling and for symmetrical Kawase passes
float4 Resample2x2Main(in noperspective float4 InUVAndScreenPos : TEXCOORD0, in FStereoPSInput StereoInput) : SV_Target0
{
	float2 UV[4];

	float2 MinUV = UVBounds.xy;
	float2 MaxUV = UVBounds.zw;

	// Shader params X/Y stores the UV offset in each direction
	UV[0] = clamp(InUVAndScreenPos.xy + ShaderParams.xy * float2(-1, -1), MinUV, MaxUV);
	UV[1] = clamp(InUVAndScreenPos.xy + ShaderParams.xy * float2(+1, -1), MinUV, MaxUV);
	UV[2] = clamp(InUVAndScreenPos.xy + ShaderParams.xy * float2(-1, +1), MinUV, MaxUV);
	UV[3] = clamp(InUVAndScreenPos.xy + ShaderParams.xy * float2(+1, +1), MinUV, MaxUV);

	float4 Sample[4];

	UNROLL for(int i = 0; i < 4; ++i)
	{
		Sample[i] = Texture2DSample(ElementTexture, ElementTextureSampler, UV[i]);
	}

	return 0.25 * float4(Sample[0] + Sample[1] + Sample[2] + Sample[3]);
}

float2 Map(float2 value, float2 min1, float2 max1, float2 min2, float2 max2)
{
  return min2 + (value - min1) * (max2 - min2) / (max1 - min1);
}

#define UPSAMPLE_OUTPUT_FORMAT_SDR 0
#define UPSAMPLE_OUTPUT_FORMAT_HDR_SCRGB 1
#define UPSAMPLE_OUTPUT_FORMAT_HDR_PQ10 2

void UpsampleMain(in noperspective float4 InUVAndScreenPos : TEXCOORD0
	, in FStereoPSInput StereoInput
	, out float4 Color0 : SV_Target0
#if UPSAMPLE_OUTPUT_FORMAT != UPSAMPLE_OUTPUT_FORMAT_SDR
	, out float4 Color1 : SV_Target1
#endif
)
{
	const float2 LocalSize = ShaderParams.xy;
	const float2 UV = InUVAndScreenPos.xy;
	const float Thickness = 0;
	const float4 CornerRadii = ShaderParams2;
	const float4 FillColor = Texture2DSample(ElementTexture, ElementTextureSampler, UV);
	Color0 = GetRoundedBoxElementColorInternal(LocalSize, Map(UV, float2(0,0), ShaderParams.zw, float2(0,0),float2(1,1)), Thickness, CornerRadii, FillColor, FillColor);

#if UPSAMPLE_OUTPUT_FORMAT != UPSAMPLE_OUTPUT_FORMAT_SDR
#if UPSAMPLE_OUTPUT_FORMAT == UPSAMPLE_OUTPUT_FORMAT_HDR_SCRGB
	static const float3x3 XYZ_2_sRGB_MAT =
	{
		 3.2409699419, -1.5373831776, -0.4986107603,
		-0.9692436363,  1.8759675015,  0.0415550574,
		 0.0556300797, -0.2039769589,  1.0569715142,
	};

	static const float3x3 Rec2020_2_XYZ_MAT =
	{
		0.6369736, 0.1446172, 0.1688585,
		0.2627066, 0.6779996, 0.0592938,
		0.0000000, 0.0280728, 1.0608437
	};

	const float3x3 Rec2020_2_sRGB = mul(XYZ_2_sRGB_MAT, Rec2020_2_XYZ_MAT);
	Color0.rgb = mul(Rec2020_2_sRGB, Color0.rgb);
#else
	Color0.rgb *= UE_SCRGB_WHITE_NITS / UE_PQ_WHITE_NITS;
	Color0.rgb = LinearToST2084Normalized(Color0.rgb);
#endif

	// Pass UI alpha to RT1 so the blend state can partially preserve the HDR scene at rounded corners
	Color1 = float4(0, 0, 0, Color0.a);
#endif
}