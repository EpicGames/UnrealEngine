// Copyright Epic Games, Inc. All Rights Reserved.

/** Shader types (mirrored from ESlateShader::Type in RenderingCommon.h */
#define ST_Default			0
#define ST_Border			1
#define ST_GrayscaleFont	2
#define ST_ColorFont		3
#define ST_LineSegment		4
#define ST_Custom			5
#define ST_PostProcess		6
#define ST_RoundedBox		7
#define ST_SdfFont			8
#define ST_MsdfFont			9
#define ST_Dynamic			10 // Special variant that uses dynamic branches for all of the above shader types

#if USE_MATERIALS
#include "/Engine/Generated/Material.ush"

	#if NUM_MATERIAL_TEXCOORDS
		#define NUM_SLATE_TEXCOORDS NUM_MATERIAL_TEXCOORDS
	#else
		#define NUM_SLATE_TEXCOORDS 1
	#endif // NUM_MATERIAL_TEXCOORDS

	#define HAS_MATERIAL_TEXCOORDS NUM_MATERIAL_TEXCOORDS > 0
#else
	#define HAS_MATERIAL_TEXCOORDS 0
	// This is defined automatically based on the number of textures in a material
	// In the case that this is a basic texture lookup shader, we only need 1 uv set
	#define NUM_SLATE_TEXCOORDS 1
#endif

#define SOURCE_IN_LINEAR_SPACE  1 

struct VertexToPixelInterpolants
{
	float4 Position : SV_POSITION;
	half4 Color : COLOR0;
	half4 SecondaryColor : COLOR1;
	float4 OriginalPosition : ORIGINAL_POSITION;
	float2 MaterialTexCoords : TEXCOORD0;
	
	float4 TextureCoordinates0 : TEXCOORD1;
#if NUM_SLATE_TEXCOORDS > 2
	float4 TextureCoordinates1 : TEXCOORD2;
#endif
#if NUM_SLATE_TEXCOORDS > 4
	float4 TextureCoordinates2 : TEXCOORD3;
#endif
#if NUM_SLATE_TEXCOORDS > 6
	float4 TextureCoordinates3 : TEXCOORD4;
#endif
};

#define SLATE_UV0(Interpolants) Interpolants.TextureCoordinates0.xy
#define SLATE_UV1(Interpolants) Interpolants.TextureCoordinates0.zw
#define SLATE_UV2(Interpolants) Interpolants.TextureCoordinates1.xy
#define SLATE_UV3(Interpolants) Interpolants.TextureCoordinates1.zw
#define SLATE_UV4(Interpolants) Interpolants.TextureCoordinates2.xy
#define SLATE_UV5(Interpolants) Interpolants.TextureCoordinates2.zw
#define SLATE_UV6(Interpolants) Interpolants.TextureCoordinates3.xy
#define SLATE_UV7(Interpolants) Interpolants.TextureCoordinates3.zw

float cross(float2 a, float2 b)
{
	return a.x*b.y - a.y*b.x;
}

float GetRoundedBoxDistance(float2 pos, float2 center, float radius, float inset)
{
	// distance from center
    pos = abs(pos - center); 

    // distance from the inner corner
    pos = pos - (center - float2(radius + inset, radius + inset));

    // use distance to nearest edge when not in quadrant with radius
    // this handles an edge case when radius is very close to thickness
    // otherwise we're in the quadrant with the radius, 
    // just use the analytic signed distance function
    return lerp( length(pos) - radius, max(pos.x - radius, pos.y - radius), float(pos.x <= 0 || pos.y <=0) );
}

float4 GetRoundedBoxElementColorInternal(float2 size, float2 UV, float thickness, float4 cornerRadii, float4 fillColor, float4 borderColor)
{
	float2 pos = size * UV;
	float2 center = size / 2.0;

	//cornerRadii: X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left */

	// figure out which radius to use based on which quadrant we're in
	float2 quadrant = step(UV, float2(.5,.5));

	float left = lerp(cornerRadii.y, cornerRadii.x, quadrant.x);
	float right = lerp(cornerRadii.z, cornerRadii.w, quadrant.x);
	float radius = lerp(right, left, quadrant.y);

	// Compute the distances internal and external to the border outline
	float dext = GetRoundedBoxDistance(pos, center, radius, 0.0);
	float din  = GetRoundedBoxDistance(pos, center, max(radius - thickness, 0), thickness);

	// Compute the border intensity and fill intensity with a smooth transition
	float bi_spread = 1.0;
	float bi = smoothstep(bi_spread, -bi_spread, dext);

	float fi_spread = .5;
	float fi = smoothstep(fi_spread, -fi_spread, din);

	float4 fill = fillColor;
	float4 border = borderColor;

	// alpha blend the external color 
	float4 OutColor = lerp(border, fill, float(thickness > radius));
	OutColor.a = 0.0;

	// blend in the border and fill colors
	OutColor = lerp(OutColor, border, bi);
	OutColor = lerp(OutColor, fill, fi);
	return OutColor;
}