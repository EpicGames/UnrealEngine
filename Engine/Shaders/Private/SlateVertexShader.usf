// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "SlateShaderCommon.ush"

VertexToPixelInterpolants Main(
	in float4 InTextureCoordinates : ATTRIBUTE0,
	in float2 InMaterialTextureCoordinates : ATTRIBUTE1,
	in float2 InPosition : ATTRIBUTE2,
	in half4 InColor : ATTRIBUTE3,
	in half4 InSecondaryColor : ATTRIBUTE4,
#if USE_SLATE_INSTANCING
	in float4 InstanceParam : ATTRIBUTE5
#else
    in uint2 PixelSize : ATTRIBUTE5
#endif
	)
{
	VertexToPixelInterpolants VOut = (VertexToPixelInterpolants)0;

    float4 WorldPosition = float4(InPosition.xy, 0, 1);

	// save off the original input position value, Line AA will require this
	VOut.OriginalPosition = WorldPosition;

#if SOURCE_IN_LINEAR_SPACE
	InColor.rgb = sRGBToLinear(InColor.rgb);
	InSecondaryColor.rgb = sRGBToLinear(InSecondaryColor.rgb);
#endif

	VOut.MaterialTexCoords = InMaterialTextureCoordinates;
	VOut.Color = InColor FCOLOR_COMPONENT_SWIZZLE;;
	VOut.SecondaryColor = InSecondaryColor FCOLOR_COMPONENT_SWIZZLE;
	VOut.TextureCoordinates0 = InTextureCoordinates;

#if USE_MATERIALS
	FMaterialVertexParameters VertexParameters = MakeInitializedMaterialVertexParameters();
	VertexParameters.WorldPosition = WorldPosition.xyz;

	VertexParameters.VertexColor = VOut.Color;
	VertexParameters.LWCData = MakeMaterialLWCData(VertexParameters);

	bool bCalledEvaluateVertexMaterialAttributes = false;

#if NUM_MATERIAL_TEXCOORDS_VERTEX

#if !USE_SLATE_INSTANCING
	// pass along local screen space size
	// To look this up in a material you use uv channel 3
	float4 InstanceParam = float4(0,0, PixelSize);
#endif

	float2 UVArrays[8] =
	{
		InTextureCoordinates.xy, //uv channel 0
		InTextureCoordinates.zw, //uv channel 1
		InstanceParam.xy, //uv channel 2
		InstanceParam.zw, //uv channel 3
		InMaterialTextureCoordinates, //uv channel 4
		float2(0,0), //uv channel 5
		float2(0,0), //uv channel 6
		float2(0,0)  //uv channel 7
	};

	{
		UNROLL
		for( uint CoordinateIndex = 0; CoordinateIndex < NUM_MATERIAL_TEXCOORDS_VERTEX; CoordinateIndex++ )
		{
			VertexParameters.TexCoords[CoordinateIndex] = UVArrays[CoordinateIndex];
		}
	}

	#if HAS_MATERIAL_TEXCOORDS

	#if ENABLE_NEW_HLSL_GENERATOR
		// This needs to be called after MakeMaterialLWCData, so that GetWorldPosition can be used :
		EvaluateVertexMaterialAttributes(VertexParameters);
		bCalledEvaluateVertexMaterialAttributes = true;
	#endif

		float2 CustomizedUVs[NUM_SLATE_TEXCOORDS];
		GetMaterialCustomizedUVs(VertexParameters, CustomizedUVs);
		
		SLATE_UV0(VOut) = CustomizedUVs[0];
	#if NUM_SLATE_TEXCOORDS > 1
		SLATE_UV1(VOut) = CustomizedUVs[1];
	#endif
	#if NUM_SLATE_TEXCOORDS > 2
		SLATE_UV2(VOut) = CustomizedUVs[2];
	#endif
	#if NUM_SLATE_TEXCOORDS > 3
		SLATE_UV3(VOut) = CustomizedUVs[3];
	#endif
	#if NUM_SLATE_TEXCOORDS > 4
		SLATE_UV4(VOut) = CustomizedUVs[4];
	#endif
	#if NUM_SLATE_TEXCOORDS > 5
		SLATE_UV5(VOut) = CustomizedUVs[5];
	#endif
	#if NUM_SLATE_TEXCOORDS > 6
		SLATE_UV6(VOut) = CustomizedUVs[6];
	#endif
	#if NUM_SLATE_TEXCOORDS > 7
		SLATE_UV7(VOut) = CustomizedUVs[7];
	#endif
		
	#endif // HAS_MATERIAL_TEXCOORDS
#endif // NUM_MATERIAL_TEXCOORDS_VERTEX

#if HAS_SCREEN_POSITION

#if ENABLE_NEW_HLSL_GENERATOR
	if (!bCalledEvaluateVertexMaterialAttributes)
	{
		EvaluateVertexMaterialAttributes(VertexParameters);
	}
#endif

	WorldPosition.xyz = GetMaterialWorldPositionOffsetRaw(VertexParameters);
#endif

#endif // USE_MATERIALS

	VOut.Position = mul(WorldPosition, SlateView.ViewProjection);

	return VOut;
}
