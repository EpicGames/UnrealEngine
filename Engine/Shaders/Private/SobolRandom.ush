// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


///////
// Typical usage of the Sobol functions for one or more points per pixel
//   uint2 SobolBase = SobolPixel(uint2(SvPosition.xy));                    // First sample for this pixel
//   for(int i = 0; i < N; ++i)
//     float2 Point = float2(SobolIndex(SobolBase, i)) / 0x10000;           // Points as [0,1) x [0,1)
//
// Typical usage for one or more points per frame
//   uint2 SobolBase = SobolPixel(uint2(SvPosition.xy));                    // Frame 0, point 0 for this pixel
//   uint2 SobolFrame = SobolIndex(SobolBase, View.StateFrameIndexMod8, 3); // Least significant bits for frame
//   for(int i = 0; i < N; ++i)
//     float2 Point = float2(SobolIndex(SobolFrame, i << 3)) / 0x10000;     // Higher-order bits for point within frame
//
// For additional independent point sets
//   uint2 SobolBase2 = SobolBase ^ RandSeed; // where RandSeed is a uint2 with values in 0 to 0xffff

// Compute a Sobol-distributed point from a 256x256 pixel grid one pixel in that grid
// @param Pixel Pixel/cell position in the 256x256 grid
// @return Sobol position relative to the pixel corner, with components in the range 0 to 0xffff
uint2 SobolPixel(uint2 Pixel)
{
	// look up for pixel
	int3 SobolLo = int3(Pixel & 0xfu, 0);
	int3 SobolHi = int3((Pixel >> 4u) & 0xfu, 0) + int3(16, 0, 0);
	uint Packed = View.SobolSamplingTexture.Load(SobolLo) ^ View.SobolSamplingTexture.Load(SobolHi);
	return uint2(Packed, Packed << 8u) & 0xff00u;
}

// Evaluate additional Sobol points within a pixel
// @param Base  Base Sobol point for this pixel
// @param Index Which 2D Sobol point to return
// @param Bits  Optional max bits in index (to avoid extra calculation)
// @return Sobol position in the range 0 to 0xffff
uint2 SobolIndex(uint2 Base, int Index, int Bits = 10)
{
	uint2 SobolNumbers[10] = {
		uint2(0x8680u, 0x4c80u), uint2(0xf240u, 0x9240u), uint2(0x8220u, 0x0e20u), uint2(0x4110u, 0x1610u), uint2(0xa608u, 0x7608u),
		uint2(0x8a02u, 0x280au), uint2(0xe204u, 0x9e04u), uint2(0xa400u, 0x4682u), uint2(0xe300u, 0xa74du), uint2(0xb700u, 0x9817u),
	};

	uint2 Result = Base;
	UNROLL for (int b = 0; b < 10 && b < Bits; ++b)
	{
		Result ^= (Index & (1u << b)) ? SobolNumbers[b] : 0;
	}
	return Result;
}

/** Returns an unique sobol sample for the frame. */
uint2 ComputePixelUniqueSobolRandSample(uint2 PixelCoord)
{
	const uint TemporalBits = 10;
	uint FrameIndexMod1024 = ReverseBitsN(GetPowerOfTwoModulatedFrameIndex(1u << TemporalBits), TemporalBits);

	uint2 SobolBase = SobolPixel(PixelCoord);
	return SobolIndex(SobolBase, FrameIndexMod1024, TemporalBits);
}

/** Maps a sobol random generated sample to 2D unit square. */
float2 SobolIndexToUniformUnitSquare(uint2 SobolRand)
{
	return float2(SobolRand) * rcp(65536.0) + rcp(65536.0 * 2.0);
}
