/************************************************************************************

Copyright   :   Copyright 2013 Oculus VR, Inc. All Rights reserved.

Licensed under the Oculus VR SDK License Version 2.0 (the "License");
you may not use the Oculus VR SDK except in compliance with the License,
which is provided at the time of installation or download, or which
otherwise accompanies this software in either electronic or hard copy form.

You may obtain a copy of the License at

http://www.oculusvr.com/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, the Oculus VR SDK
distributed under the License is distributed on an "AS IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.

*************************************************************************************/
#include "Common.ush"

void Main(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
    OutColor = float4(Input.UV.x, Input.UV.y, Input.UV.x, 1);
}
