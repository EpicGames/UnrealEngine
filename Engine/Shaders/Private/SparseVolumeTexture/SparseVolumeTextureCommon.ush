// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../RandomInterleavedGradientNoise.ush"

// SVT_TODO: Unify with macros in SparseVolumeTexture.h
#define SPARSE_VOLUME_TILE_RES 16
#define SPARSE_VOLUME_TILE_BORDER 1
#define SPARSE_VOLUME_TILE_RES_PADDED (SPARSE_VOLUME_TILE_RES + 2 * SPARSE_VOLUME_TILE_BORDER)

#define SVTADDRESSMODE_CLAMP 0u
#define SVTADDRESSMODE_WRAP 1u
#define SVTADDRESSMODE_MIRROR 2u

#ifndef SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING
#define SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING 1 // SVT_TODO: Default to stochastic mip sampling once Heterogeneous Volumes properly initializes FMaterialPixelParameters.
#endif

float SparseVolumeTextureApplyAddressModeMirror(float v)
{
	float t = frac(v * 0.5f) * 2.0f;
	return 1.0f - abs(t - 1.0f);
}

float SparseVolumeTextureApplyAddressMode(float v, uint AddressMode)
{
	// For CLAMP address mode, can't clamp to 1.0f, otherwise 'int(UVW * VolumePageResolution)' might overflow page table bounds by 1
	// Instead, clamp to slightly below 1, this ensures that when rounded down to int, above value will be at most 'PageTableResolution - 1'
	// The actual texel we clamp to doesn't matter too much for sampling physical texture, since we have borders around the physical pages
	// Just need to make sure we don't clamp too far and chop off valid texels at the edge of texture
	const float MaxTextureSize = 65536.0f;

	switch (AddressMode)
	{
	case SVTADDRESSMODE_WRAP: return frac(v);
	case SVTADDRESSMODE_MIRROR: return SparseVolumeTextureApplyAddressModeMirror(v);
	default: return clamp(v, 0.0f, 1.0f - (1.0f / MaxTextureSize));
	}
}

float3 SparseVolumeTextureApplyAddressMode(float3 UVW, uint AddressU, uint AddressV, uint AddressW)
{
	return float3(
		SparseVolumeTextureApplyAddressMode(UVW.x, AddressU),
		SparseVolumeTextureApplyAddressMode(UVW.y, AddressV),
		SparseVolumeTextureApplyAddressMode(UVW.z, AddressW));
}

struct FSparseVolumeTextureUniforms
{
	float3			VolumePageResolution;
	float3			PageTableOffset;
	float3			TileDataTexelSize;
	int				FrameIndex;
	int             HighestMipLevel;
};

FSparseVolumeTextureUniforms SparseVolumeTextureUnpackUniforms(const uint4 Packed0, const uint4 Packed1)
{
	FSparseVolumeTextureUniforms Result;
	Result.VolumePageResolution = asfloat(Packed0.xyz);
	Result.PageTableOffset.x = float(Packed0.w & 0x7FFu);
	Result.PageTableOffset.y = float((Packed0.w >> 11u) & 0x7FFu);
	Result.PageTableOffset.z = float((Packed0.w >> 22u) & 0x3FFu);
	Result.TileDataTexelSize = asfloat(Packed1.xyz);
	Result.FrameIndex = int(Packed1.w & 0xFFFFu);
	Result.HighestMipLevel = int((Packed1.w >> 16u) & 0xFFFFu);

	return Result;
}

float SparseVolumeTextureCalculateMipLevel(FSparseVolumeTextureUniforms Uniforms, float3 UVWDDX, float3 UVWDDY, float MipBias, float2 SvPositionXY)
{
	const float3 Resolution = Uniforms.VolumePageResolution * float(SPARSE_VOLUME_TILE_RES);
	const float3 TexCoordDDX = UVWDDX * Resolution;
	const float3 TexCoordDDY = UVWDDY * Resolution;
	const float DDXLengthSquared = dot(TexCoordDDX, TexCoordDDX);
	const float DDYLengthSquared = dot(TexCoordDDY, TexCoordDDY);
	const float MaxLengthSquared = max(DDXLengthSquared, DDYLengthSquared);
	const float ComputedMipLevel = 0.5f * log2(max(MaxLengthSquared, 1e-8f));

#if SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING
	const float Noise = 0.0f;
#else
	const float Noise = InterleavedGradientNoise(SvPositionXY, View.StateFrameIndexMod8);
#endif

	const float FinalMipLevelClamped = max(0.0f, ComputedMipLevel + MipBias + Noise);
	return FinalMipLevelClamped;
}

float3 SparseVolumeTextureGetVoxelCoord(Texture3D<uint> PageTable, FSparseVolumeTextureUniforms Uniforms, float3 PageTableCoord, int MipLevel)
{
	// Load from page table texture
	const uint PackedPhysicalTileCoord = PageTable.Load(int4(floor(PageTableCoord), MipLevel)).x;
	const int3 PhysicalTileCoord = int3(PackedPhysicalTileCoord & 0xFFu, (PackedPhysicalTileCoord >> 8u) & 0xFFu, (PackedPhysicalTileCoord >> 16u) & 0xFFu);
	const uint TileMipLevel = PackedPhysicalTileCoord >> 24u;

	// Scale TileCoord by 1 / exp2(ActualMipLevel - DesiredMipLevel) to get correct UVs when sampling from a higher mip as fallback
	const float TileRcpMipLevelFactor = rcp(float(1u << (TileMipLevel - uint(MipLevel))));
	const float3 FracTileCoord = frac(PageTableCoord * TileRcpMipLevelFactor);

	const float3 VoxelCoord = float3(PhysicalTileCoord) * float(SPARSE_VOLUME_TILE_RES_PADDED) + (FracTileCoord * float(SPARSE_VOLUME_TILE_RES) + float(SPARSE_VOLUME_TILE_BORDER));
	return VoxelCoord;
}

float3 SparseVolumeTextureSamplePageTable(Texture3D<uint> PageTable, FSparseVolumeTextureUniforms Uniforms, float3 UVW, uint AddressU, uint AddressV, uint AddressW, int MipLevel = 0)
{
	// Apply address mode to UVW and clamp mip level to resident levels
	UVW = SparseVolumeTextureApplyAddressMode(UVW, AddressU, AddressV, AddressW);
	MipLevel = clamp(MipLevel, 0, Uniforms.HighestMipLevel);

	const float RcpMipLevelFactor = rcp(float(1u << (uint)MipLevel));
	const float3 VolumePageCoord = UVW * Uniforms.VolumePageResolution;
	const float3 MipPageTableOffset = floor(Uniforms.PageTableOffset * RcpMipLevelFactor);
	const float3 PageTableCoord = VolumePageCoord * RcpMipLevelFactor - MipPageTableOffset;

	const float3 VoxelCoord = SparseVolumeTextureGetVoxelCoord(PageTable, Uniforms, PageTableCoord, MipLevel);
	const float3 VoxelUVW = VoxelCoord * Uniforms.TileDataTexelSize;
	return VoxelUVW;
}

int3 SparseVolumeTextureLoadPageTable(Texture3D<uint> PageTable, FSparseVolumeTextureUniforms Uniforms, int3 TexelCoord, int MipLevel = 0)
{
	if (MipLevel < 0 || MipLevel > Uniforms.HighestMipLevel)
	{
		return 0; // Point to null tile
	}
	const float RcpMipLevelFactor = rcp(float(1u << (uint)MipLevel));
	const float3 VolumeMipPageCoord = (TexelCoord + 0.5f) / float(SPARSE_VOLUME_TILE_RES);
	const float3 MipPageTableOffset = floor(Uniforms.PageTableOffset * RcpMipLevelFactor);
	const float3 PageTableCoord = VolumeMipPageCoord - MipPageTableOffset;

	const float3 VoxelCoord = SparseVolumeTextureGetVoxelCoord(PageTable, Uniforms, PageTableCoord, MipLevel);
	return int3(VoxelCoord);
}

float4 SparseVolumeTextureSamplePhysicalTileData(Texture3D PhysicalTileDataA, Texture3D PhysicalTileDataB, SamplerState TileDataSampler, float3 VoxelUVW, int PhysicalTileDataIndex)
{
	switch (PhysicalTileDataIndex)
	{
	case 0: return PhysicalTileDataA.SampleLevel(TileDataSampler, VoxelUVW, 0.0f);
	case 1: return PhysicalTileDataB.SampleLevel(TileDataSampler, VoxelUVW, 0.0f);
	default: return 0.0f;
	}
}

float3 SparseVolumeTextureSamplePageTableSecondMipWrapper(Texture3D<uint> PageTable, FSparseVolumeTextureUniforms Uniforms, float3 UVW, uint AddressU, uint AddressV, uint AddressW, int MipLevel = 0)
{
#if SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING
	return SparseVolumeTextureSamplePageTable(PageTable, Uniforms, UVW, AddressU, AddressV, AddressW, MipLevel);
#else
	return 0.0f;
#endif
}

float4 SparseVolumeTextureSamplePhysicalTileDataSecondMipWrapper(Texture3D PhysicalTileDataA, Texture3D PhysicalTileDataB, SamplerState TileDataSampler, float3 VoxelUVW, int PhysicalTileDataIndex)
{
#if SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING
	return SparseVolumeTextureSamplePhysicalTileData(PhysicalTileDataA, PhysicalTileDataB, TileDataSampler, VoxelUVW, PhysicalTileDataIndex);
#else
	return 0.0f;
#endif
}

float4 SparseVolumeTextureCombineMipSamples(float4 LowerMipSample, float4 UpperMipSample, float MipLevelFractionalPart)
{
#if SPARSE_VOLUME_TEXTURE_SOFTWARE_LINEAR_MIP_FILTERING
	return lerp(LowerMipSample, UpperMipSample, MipLevelFractionalPart);
#else
	return LowerMipSample;
#endif
}
