// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush" 
#include "SparseVolumeTextureCommon.ush"

uint FlattenCoord(uint3 Coord, uint3 VolumeSize)
{
	return Coord.z * (VolumeSize.y * VolumeSize.x) + Coord.y * VolumeSize.x + Coord.x;
}

uint3 UnflattenCoord(uint Index, uint3 VolumeSize)
{
	const uint SliceIndex = Index / (VolumeSize.y * VolumeSize.x);
	const uint SliceLocalIndex = Index - SliceIndex * (VolumeSize.y * VolumeSize.x);

	uint3 Result;
	Result.z = SliceIndex;
	Result.y = SliceLocalIndex / VolumeSize.x;
	Result.x = SliceLocalIndex - (Result.y * VolumeSize.x);

	return Result;
}

#ifdef UPDATE_TILE_TEXTURE_FROM_SPARSE_BUFFER

#ifndef TEXTURE_UPDATE_MASK
#define TEXTURE_UPDATE_MASK 0
#endif // TEXTURE_UPDATE_MASK

#define UPDATE_TEXTURE_A ((TEXTURE_UPDATE_MASK & 1) != 0)
#define UPDATE_TEXTURE_B ((TEXTURE_UPDATE_MASK & 2) != 0)

#if UPDATE_TEXTURE_A
RWTexture3D<float4> DstPhysicalTileTextureA;
Buffer<float4> SrcPhysicalTileBufferA;
#endif // UPDATE_TEXTURE_A

#if UPDATE_TEXTURE_B
RWTexture3D<float4> DstPhysicalTileTextureB;
Buffer<float4> SrcPhysicalTileBufferB;
#endif // UPDATE_TEXTURE_B

ByteAddressBuffer OccupancyBitsBuffer;
ByteAddressBuffer TileDataOffsetsBuffer;
ByteAddressBuffer DstTileCoordsBuffer;
float4 FallbackValueA;
float4 FallbackValueB;
uint TileIndexOffset; // Number of tiles that have already been processed in previous dispatches
uint SrcVoxelDataOffsetA; // Start of the SrcPhysicalTileBufferA SRV. This value needs to be subtracted from any indices used for loads.
uint SrcVoxelDataOffsetB; // Start of the SrcPhysicalTileBufferB SRV. This value needs to be subtracted from any indices used for loads.
uint NumTilesToCopy;
uint BufferTileStep;
uint NumDispatchedGroups;
uint PaddedTileSize;

uint ComputeLocalVoxelDataLoadIndex(uint VoxelIndex, uint2 OccupancyBits, out bool bValidVoxel)
{
	VoxelIndex &= 63u;
	bValidVoxel = false;
	uint PrefixSum = 0;
	if (VoxelIndex < 32u)
	{
		bValidVoxel = OccupancyBits.x & (1u << VoxelIndex);
		PrefixSum = countbits(OccupancyBits.x & ((1u << VoxelIndex) - 1u));
	}
	else
	{
		const uint WrappedVoxelIndex = VoxelIndex - 32u;
		bValidVoxel = OccupancyBits.y & (1u << WrappedVoxelIndex);
		PrefixSum = countbits(OccupancyBits.x) + countbits(OccupancyBits.y & ((1u << WrappedVoxelIndex) - 1u));
	}
	return PrefixSum;
}

void CopyToTileDataTexture(RWTexture3D<float4> DstTileTexture, Buffer<float4> SrcBuffer, uint SrcVoxelDataOffset, uint TileIdx, uint TextureTileOffset, uint3 DstBaseCoord, float4 FallbackValue, uint GroupThreadIndex)
{
	const uint NumVoxelsPerPaddedTile = PaddedTileSize * PaddedTileSize * PaddedTileSize;
	const uint NumOccupancyWordsPerPaddedTile = (NumVoxelsPerPaddedTile + 31u) / 32u;

	// The offset in the source buffer at which the tile data for this tile starts.
	const uint TileDataOffset = TileDataOffsetsBuffer.Load((TileIdx + TextureTileOffset) << 2u);
	uint OccupancyLoadIdx = (TileIdx + TextureTileOffset) * NumOccupancyWordsPerPaddedTile;
	// Keep track of the number of voxels read from the source buffer (for this tile only).
	uint NumReadVoxels = 0;

	// Iterate over all voxels in the tile and copy them.
	for (uint VoxelIdx = GroupThreadIndex; VoxelIdx < NumVoxelsPerPaddedTile; VoxelIdx += 64)
	{
		const uint2 OccupancyBits = OccupancyBitsBuffer.Load2(OccupancyLoadIdx << 2u);

		bool bValidVoxel = false;
		const uint LocalLoadIndex = ComputeLocalVoxelDataLoadIndex(VoxelIdx, OccupancyBits, bValidVoxel);
		const uint VoxelLoadIndex = TileDataOffset + NumReadVoxels + LocalLoadIndex;
		const uint3 DstCoord = DstBaseCoord + UnflattenCoord(VoxelIdx, PaddedTileSize);

		float4 Voxel = FallbackValue;
		BRANCH if (bValidVoxel)
		{
			Voxel = SrcBuffer.Load(VoxelLoadIndex - SrcVoxelDataOffset);
		}
		DstTileTexture[DstCoord] = Voxel;

		OccupancyLoadIdx += 2; // We just consumed 64 bits (2 uints)
		NumReadVoxels += countbits(OccupancyBits.x) + countbits(OccupancyBits.y);
	}
}

[numthreads(64, 1, 1)]
void SparseVolumeTextureUpdateFromSparseBufferCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint GroupIdx = GroupId.x;

	// Process one tile per workgroup
	for (uint TileIdx = (GroupIdx + TileIndexOffset); TileIdx < (NumTilesToCopy + TileIndexOffset); TileIdx += NumDispatchedGroups)
	{
		const uint PackedDstCoord = DstTileCoordsBuffer.Load(TileIdx << 2u);
		const uint3 DstBaseCoord = uint3(PackedDstCoord & 0xFFu, (PackedDstCoord >> 8u) & 0xFFu, (PackedDstCoord >> 16u) & 0xFFu) * PaddedTileSize;

		// The start offset of the voxels and the occupancy bits of any given tile are stored per texture in a contiguous section of the buffer(s).
		// This offset is used to jump to the offset for the current texture.
		uint TextureTileOffset = 0;

#if UPDATE_TEXTURE_A
		CopyToTileDataTexture(DstPhysicalTileTextureA, SrcPhysicalTileBufferA, SrcVoxelDataOffsetA, TileIdx, TextureTileOffset, DstBaseCoord, FallbackValueA, GroupThreadIndex);
		TextureTileOffset += BufferTileStep;
#endif // UPDATE_TEXTURE_A

#if UPDATE_TEXTURE_B
		CopyToTileDataTexture(DstPhysicalTileTextureB, SrcPhysicalTileBufferB, SrcVoxelDataOffsetB, TileIdx, TextureTileOffset, DstBaseCoord, FallbackValueB, GroupThreadIndex);
#endif // UPDATE_TEXTURE_B
	}
}

#endif // UPDATE_TILE_TEXTURE_FROM_SPARSE_BUFFER

#ifdef UPDATE_PAGE_TABLE

RWTexture3D<uint> PageTable;
ByteAddressBuffer PageTableUpdates;
uint UpdateCoordOffset;
uint UpdatePayloadOffset;
uint NumUpdates;

[numthreads(64, 1, 1)]
void SparseVolumeTextureUpdatePageTableCS(uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x < NumUpdates)
	{
		// The page table coordinate indexes into the page table itself and needs to be able to address each individual voxel.
		// As the maximum dimension of 3D textures is 2048, we are using a 11|11|10 bit packing scheme for the page table coordinate.
		const uint PackedPageTableCoord = PageTableUpdates.Load((DispatchThreadID.x + UpdateCoordOffset) << 2);
		const uint3 PageTableCoord = uint3(PackedPageTableCoord & 0x7FFu, (PackedPageTableCoord >> 11u) & 0x7FFu, (PackedPageTableCoord >> 22u) & 0x3FFu);

		// The payload is the data that is written to the page table. It stores the coordinate of the pointed to tile
		// in the physical tile data texture and is encoded as XYZ with 8 bit per component, taking up 24 bits and leaving 8 bit for other uses.
		// 8 bit is enough for this as we only need to address entire tiles, not individual voxels of the physical tile data texture.
		const uint Payload = PageTableUpdates.Load((DispatchThreadID.x + UpdatePayloadOffset) << 2);
		
		PageTable[PageTableCoord] = Payload;
	}
}

#endif // UPDATE_PAGE_TABLE
