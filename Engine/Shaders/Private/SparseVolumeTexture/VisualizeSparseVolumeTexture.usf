// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VisualizeSparseVolumeTexture.usf: Used to visualize and debug a sparse texture asset.
=============================================================================*/

#include "/Engine/Private/Common.ush"
#include "SparseVolumeTextureCommon.ush"


#ifdef VERTEX_SHADER

float DepthAsDeviceZ;

void VisualizeSparseVolumeTextureVS(
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION)
{
	float2 UV = -1.0f;
	UV = VertexId == 1 ? float2(-1.0f, 3.0f) : UV;
	UV = VertexId == 2 ? float2(3.0f, -1.0f) : UV;

	Position = float4(UV, DepthAsDeviceZ, 1.0f);
}

#endif // VERTEX_SHADER


#ifdef PIXEL_SHADER

// Updated from http://jcgt.org/published/0007/03/04/
bool Slabs(float3 p0, float3 p1, float3 rayOrigin, float3 invRaydir, out float outTMin, out float outTMax)
{
	float3 t0 = (p0 - rayOrigin) * invRaydir;
	float3 t1 = (p1 - rayOrigin) * invRaydir;
	float3 tmin = min(t0, t1), tmax = max(t0, t1);
	float maxtmin = max(max(tmin.x, tmin.y), tmin.z);
	float mintmax = min(min(tmax.x, tmax.y), tmax.z);
	outTMin = maxtmin;
	outTMax = mintmax;
	return maxtmin <= mintmax;
}

SamplerState TileDataTextureSampler;
Texture3D<uint> SparseVolumeTexturePageTable;
Texture3D<float4> SparseVolumeTextureA;
Texture3D<float4> SparseVolumeTextureB;
uint4 PackedSVTUniforms0;
uint4 PackedSVTUniforms1;
float3 SparseVolumeTextureResolution;
int MipLevel;
float4 WorldToLocal0;
float4 WorldToLocal1;
float4 WorldToLocal2;
float3 WorldToLocalRotation0;
float3 WorldToLocalRotation1;
float3 WorldToLocalRotation2;
uint ComponentToVisualize;
float Extinction;

void VisualizeSparseVolumeTexturePS(
	in float4 SVPos : SV_POSITION,
	out float4 OutLuminanceAlpha : SV_Target0
)
{
	ResolvedView = ResolveView();

	const float3 TranslatedWorldPos = SvPositionToResolvedTranslatedWorld(SVPos);
	const float3 RayDirWorld = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPos);
	const float3 RayOriginWorld = SvPositionToResolvedTranslatedWorld(float4(SVPos.xy, NearDepthValue, 1.0f));

	// Transform into local [-1.0f, 1.0f] space
	float3 RayDir;
	RayDir.x = dot(WorldToLocal0.xyz, RayDirWorld);
	RayDir.y = dot(WorldToLocal1.xyz, RayDirWorld);
	RayDir.z = dot(WorldToLocal2.xyz, RayDirWorld);

	float3 RayOrig;
	RayOrig.x = dot(WorldToLocal0, float4(RayOriginWorld, 1.0f));
	RayOrig.y = dot(WorldToLocal1, float4(RayOriginWorld, 1.0f));
	RayOrig.z = dot(WorldToLocal2, float4(RayOriginWorld, 1.0f));

	OutLuminanceAlpha = float4(0.0, 0.0, 0.0, 1.0);

	float TMin = 0.0f;
	float TMax = 0.0f;
	float Transmittance = 1.0f;
	if (Slabs(-1.0f, 1.0f, RayOrig, 1.0f / RayDir, TMin, TMax) && TMax > 0.0f)
	{
		Transmittance = 0.5; // show the bounding box

		// Amanatides 3D DDA marching implementation - Paper: http://citeseerx.ist.psu.edu/viewdoc/download?doi=10.1.1.42.3443&rep=rep1&type=pdf
		// See https://www.shadertoy.com/view/3sKXDK

		float3 StartPos = RayOrig + RayDir * max(0.0, TMin);

		float3 VolumeResolution = SparseVolumeTextureResolution;
		float3 RcpVolumeResolution = 1.0f / VolumeResolution;

		const float3 StartUVs = StartPos * 0.5f + 0.5f;
		float3 P = StartUVs * VolumeResolution;

		// Force round to volume
		if (P.x < 0) P.x = 0;
		if (P.y < 0) P.y = 0;
		if (P.z < 0) P.z = 0;
		if (P.x > (VolumeResolution.x)) P.x = (VolumeResolution.x);
		if (P.y > (VolumeResolution.y)) P.y = (VolumeResolution.y);
		if (P.z > (VolumeResolution.z)) P.z = (VolumeResolution.z);

		// DDA requires the ray direction not to be non-(uniformly) scaled.
		float3 RayDirNoScale;
		RayDirNoScale.x = dot(WorldToLocalRotation0, RayDirWorld);
		RayDirNoScale.y = dot(WorldToLocalRotation1, RayDirWorld);
		RayDirNoScale.z = dot(WorldToLocalRotation2, RayDirWorld);

		// Amanatides 3D-DDA data preparation
		float3 stepSign = sign(RayDirNoScale);
		float3 tDelta = abs(1.0f / RayDirNoScale);
		float3 tMax = float3(0.0, 0.0, 0.0);
		float3 refPoint = floor(P);
		tMax.x = stepSign.x > 0.0 ? refPoint.x + 1.0 - P.x : P.x - refPoint.x;
		tMax.y = stepSign.y > 0.0 ? refPoint.y + 1.0 - P.y : P.y - refPoint.y;
		tMax.z = stepSign.z > 0.0 ? refPoint.z + 1.0 - P.z : P.z - refPoint.z;
		tMax.x *= tDelta.x;
		tMax.y *= tDelta.y;
		tMax.z *= tDelta.z;

		const FSparseVolumeTextureUniforms SVTUniforms = SparseVolumeTextureUnpackUniforms(PackedSVTUniforms0, PackedSVTUniforms1);

		float StepLength = 1.0f;

		LOOP
		while (P.x >= 0 && P.y >= 0 && P.z >= 0 && P.x <= VolumeResolution.x && P.y <= VolumeResolution.y && P.z <= VolumeResolution.z)
		{
			const float3 UVW = P * RcpVolumeResolution;
			const float3 VoxelUVW = SparseVolumeTextureSamplePageTable(SparseVolumeTexturePageTable, SVTUniforms, UVW, SVTADDRESSMODE_CLAMP, SVTADDRESSMODE_CLAMP, SVTADDRESSMODE_CLAMP, MipLevel);
			const float3 UVWTileSize = float(SPARSE_VOLUME_TILE_RES_PADDED) * SVTUniforms.TileDataTexelSize;
			if (any(VoxelUVW >= UVWTileSize)) // skip on null tile
			{
				const int PhysicalTileDataIndex = ComponentToVisualize < 4 ? 0 : 1;
				const float4 VoxelData = SparseVolumeTextureSamplePhysicalTileData(SparseVolumeTextureA, SparseVolumeTextureB, TileDataTextureSampler, VoxelUVW, PhysicalTileDataIndex);

				float Density = 0.0f;
				if (ComponentToVisualize < 4)
				{
					Density = VoxelData[ComponentToVisualize];
				}
				else
				{
					Density = VoxelData[min(ComponentToVisualize - 4, 3u)];
				}

				if (Density > 0.0f)
				{
#if 0
					//OutLuminanceAlpha = float4(Rand3DPCG16(PageCoordF).x / 65535.0f, Rand3DPCG16(VoxelCoord).y / 65535.0f, 0.0, 0.0);
					//OutLuminanceAlpha = float4(Rand3DPCG16(PageCoordF) / 65535.0f, 0.0);
					OutLuminanceAlpha = float4(Rand3DPCG16(VoxelCoord) / 65535.0f, 0.0);
					break;
#else
					Transmittance *= exp(-StepLength * Density * Extinction);
#endif
				}
			}

#if 0
			// Slow reference
			P += RayDir * 0.005;
			StepLength = 0.005;
#else
			// Amanatides 3D-DDA 
			if (tMax.x < tMax.y)
			{
				if (tMax.x < tMax.z)
				{
					P.x += stepSign.x;
					tMax.x += tDelta.x;
				}
				else
				{
					P.z += stepSign.z;
					tMax.z += tDelta.z;
				}
			}
			else
			{
				if (tMax.y < tMax.z)
				{
					P.y += stepSign.y;
					tMax.y += tDelta.y;
				}
				else
				{
					P.z += stepSign.z;
					tMax.z += tDelta.z;
				}
			}
#endif
		}
	}

	OutLuminanceAlpha.a = Transmittance;
}


#endif // PIXEL_SHADER


