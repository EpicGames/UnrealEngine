// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush" 
#include "/Engine/Shared/SubstrateDefinitions.h"
#include "SpecularProfileCommon.ush"

///////////////////////////////////////////////////////////////////////////////////////////////////

#if defined(SHADER_COPY) || defined(SHADER_CONVOLVE)
	#if PERMUTATION_COMPONENT_COUNT == 3
		#define SPECULAR_PROFILE_TYPE float3
		#define SPECULAR_PROFILE_WRITE(RGBValue) RGBValue
	#elif PERMUTATION_COMPONENT_COUNT == 4
		#define SPECULAR_PROFILE_TYPE float4
		#define SPECULAR_PROFILE_WRITE(RGBValue) float4(RGBValue,1)
	#else
		#error Only component counts of 3 or 4 are currently supported
	#endif
#endif

#ifdef SHADER_COPY

int2 SourceResolution;
uint SourceMipCount;
int2 TargetResolution;
uint TargetIndex;
SamplerState SourceSampler;
Texture2D<SPECULAR_PROFILE_TYPE> SourceTexture;
RWTexture2DArray<SPECULAR_PROFILE_TYPE> TargetTexture;

Buffer<float4> ViewColorBuffer;
Buffer<float4> LightColorBuffer;

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = DispatchThreadId.xy;
	if (all(PixelPos < uint2(TargetResolution)))
	{
		const float2 TexCoord = float2(PixelPos + 0.5f) / float2(TargetResolution);
		#if PERMUTATION_PROCEDURAL
		{
			const float3 C0 = ViewColorBuffer[PixelPos.x].xyz;
			const float3 C1 = LightColorBuffer[PixelPos.y].xyz;
			const float3 C0C1 = C0 * C1;
			TargetTexture[uint3(PixelPos,TargetIndex * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT + SUBSTRATE_SPECULAR_PROFILE_ENTRY_LIGHT)] = SPECULAR_PROFILE_WRITE(C0C1);
		}
		#else
		{
			const float SourceMip = clamp(log2(SourceResolution.x) - log2(TargetResolution.x), 0.f, SourceMipCount-1);
			const float3 Value = SourceTexture.SampleLevel(SourceSampler, TexCoord, SourceMip).xyz;
			TargetTexture[uint3(PixelPos,TargetIndex * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT + SUBSTRATE_SPECULAR_PROFILE_ENTRY_LIGHT)] = SPECULAR_PROFILE_WRITE(Value);
		}
		#endif
	}
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef SHADER_CONVOLVE

#include "MonteCarlo.ush"
#include "BRDF.ush"

uint SampleCount;
int2 Resolution;
uint ProfileCount;
#if PERMUTATION_READ_WRITE_ON_ARRAY == 0
Texture2DArray<SPECULAR_PROFILE_TYPE> SourceTexture;
#endif
RWTexture2DArray<SPECULAR_PROFILE_TYPE> TargetTexture;
StructuredBuffer<float> ProfileSignBuffer;

float3 EvaluateSpecularProfile(float NoV, float NoL, float VoH, float NoH, float SpecularProfileId)
{
	const float3 ProfileCoord = GetSpecularProfileCoord(SpecularProfileId, NoV, NoL, VoH, NoH);
	uint3 FetchCoord = uint3(ProfileCoord.xy * (Resolution - 1), ProfileCoord.z);
	FetchCoord.x = clamp(FetchCoord.x, 0u, uint(Resolution.x - 1));
	FetchCoord.y = clamp(FetchCoord.y, 0u, uint(Resolution.y - 1));
	#if PERMUTATION_READ_WRITE_ON_ARRAY
	return TargetTexture[FetchCoord].xyz;
	#else
	return SourceTexture[FetchCoord].xyz;
	#endif
}

float3 Convolve(float3 V, float3 N, float Roughness, float SpecularProfileId)
{
	float3 OutValue = 0;
	float OutWeight = 0;

	LOOP
	for (uint SampleIt = 0; SampleIt < SampleCount; ++SampleIt)
	{
		const float2 E = Hammersley(SampleIt, SampleCount, 0);

		const float4 HAndPDF = ImportanceSampleVisibleGGX(E, Pow2(Roughness), V);
		const float3 H = HAndPDF.xyz;
		const float PDF = HAndPDF.w;

		// Sample generation
		const float3 L = 2 * dot(V, H) * H - V;
		if (L.z > 0)
		{
			const float NoV = saturate(dot(N, V));
			const float NoL = saturate(dot(N, L));
			const float NoH = saturate(dot(N, H));
			const float VoH = saturate(dot(V, H));

			if (PDF > 0)
			{
				const float3 Value = EvaluateSpecularProfile(NoV, NoL, VoH, NoH, SpecularProfileId);
				OutValue += Value;
				OutWeight+= 1;
			}
		}
	}
	OutValue /= max(0.001f, OutWeight);
	return OutValue;
}

[numthreads(8, 8, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = DispatchThreadId.xy;
	const uint ProfileIndex = DispatchThreadId.z;
	if (all(PixelPos < uint2(Resolution)) && ProfileIndex < ProfileCount)
	{
		const float NoV = (PixelPos.x+0.5f) / float(Resolution.x);
		const float Roughness = max(PixelPos.y / float(Resolution.y-1), 0.02f);
		const float SpecularProfileId = ProfileSignBuffer[ProfileIndex] * ProfileIndex; // 	The sign encodes the parameterization (NoV / NoH)

		const float3 N = float3(0, 0, 1);
		const float3 V = float3(sqrt(1-NoV*NoV), 0, NoV);
		const float3 ConvolvedColor = Convolve(V, N, Roughness, SpecularProfileId);

		const uint3 OutCoord = uint3(PixelPos, ProfileIndex * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT + SUBSTRATE_SPECULAR_PROFILE_ENTRY_ENV);
		TargetTexture[OutCoord] = SPECULAR_PROFILE_WRITE(ConvolvedColor);
	}
}
#endif
