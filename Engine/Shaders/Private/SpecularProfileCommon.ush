// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

uint GetSpecularProfileId(float In)
{
	return uint(abs(In));
}

// Return 0: View/Light angles, 1: Half angles
uint GetSpecularProfileParameterization(float In)
{
	return In > 0.f ? 0u : 1u;
}

float2 InternalGetSpecularProfileCoord(float InSpecularProfileId, float NoV, float NoL, float VoH, float NoH)
{
	const bool bUseHalfVector = GetSpecularProfileParameterization(InSpecularProfileId) == 1;
	return bUseHalfVector ? float2(VoH, NoH) : float2(NoV, NoL);
}

float3 GetSpecularProfileCoord(float InSpecularProfileId, float NoV, float NoL, float VoH, float NoH)
{
	const uint ProfileID = GetSpecularProfileId(InSpecularProfileId);
	const uint SliceIndex = ProfileID * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT + SUBSTRATE_SPECULAR_PROFILE_ENTRY_LIGHT;
	return float3(InternalGetSpecularProfileCoord(InSpecularProfileId, NoV, NoL, VoH, NoH), SliceIndex);
}

float3 GetSpecularProfileEnvCoord(float InSpecularProfileId, float NoV, float InRoughness)
{
	const uint ProfileID = GetSpecularProfileId(InSpecularProfileId);
	const uint SliceIndex = ProfileID * SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT + SUBSTRATE_SPECULAR_PROFILE_ENTRY_ENV;
	return float3(NoV, InRoughness, SliceIndex);
}