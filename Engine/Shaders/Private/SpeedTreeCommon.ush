// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * SpeedTreeCommon.usf: Code for SpeedTree vertex processing
 */

 #include "/Engine/Generated/UniformBuffers/SpeedTreeData.ush"

float3 SpeedTreeLODInfo;

// Make sure this matches the SpeedTreeData uniform buffer
struct FSpeedTreeData
{
	float4 WindVector;
	float4 WindGlobal;
	float4 WindBranch;
	float4 WindBranchTwitch;
	float4 WindBranchWhip;
	float4 WindBranchAnchor;
	float4 WindBranchAdherences;
	float4 WindTurbulences;
	float4 WindLeaf1Ripple;
	float4 WindLeaf1Tumble;
	float4 WindLeaf1Twitch;
	float4 WindLeaf2Ripple;
	float4 WindLeaf2Tumble;
	float4 WindLeaf2Twitch;
	float4 WindFrondRipple;
	float4 WindRollingBranch;
	float4 WindRollingLeafAndDirection;
	float4 WindRollingNoise;
	float4 WindAnimation;
};

FSpeedTreeData GetCurrentSpeedTreeData()
{
	// Make sure these matches all members in FSpeedTreeData
	FSpeedTreeData STData;
	STData.WindVector = SpeedTreeData.WindVector;
	STData.WindGlobal = SpeedTreeData.WindGlobal;
	STData.WindBranch = SpeedTreeData.WindBranch;
	STData.WindBranchTwitch = SpeedTreeData.WindBranchTwitch;
	STData.WindBranchWhip = SpeedTreeData.WindBranchWhip;
	STData.WindBranchAnchor = SpeedTreeData.WindBranchAnchor;
	STData.WindBranchAdherences = SpeedTreeData.WindBranchAdherences;
	STData.WindTurbulences = SpeedTreeData.WindTurbulences;
	STData.WindLeaf1Ripple = SpeedTreeData.WindLeaf1Ripple;
	STData.WindLeaf1Tumble = SpeedTreeData.WindLeaf1Tumble;
	STData.WindLeaf1Twitch = SpeedTreeData.WindLeaf1Twitch;
	STData.WindLeaf2Ripple = SpeedTreeData.WindLeaf2Ripple;
	STData.WindLeaf2Tumble = SpeedTreeData.WindLeaf2Tumble;
	STData.WindLeaf2Twitch = SpeedTreeData.WindLeaf2Twitch;
	STData.WindFrondRipple = SpeedTreeData.WindFrondRipple;
	STData.WindRollingBranch = SpeedTreeData.WindRollingBranch;
	STData.WindRollingLeafAndDirection = SpeedTreeData.WindRollingLeafAndDirection;
	STData.WindRollingNoise = SpeedTreeData.WindRollingNoise;
	STData.WindAnimation = SpeedTreeData.WindAnimation;
	return STData;
}

FSpeedTreeData GetPreviousSpeedTreeData()
{
	// Same as GetCurrentSpeedTreeData(), but using the 'Prev' members
	FSpeedTreeData STData;
	STData.WindVector = SpeedTreeData.PrevWindVector;
	STData.WindGlobal = SpeedTreeData.PrevWindGlobal;
	STData.WindBranch = SpeedTreeData.PrevWindBranch;
	STData.WindBranchTwitch = SpeedTreeData.PrevWindBranchTwitch;
	STData.WindBranchWhip = SpeedTreeData.PrevWindBranchWhip;
	STData.WindBranchAnchor = SpeedTreeData.PrevWindBranchAnchor;
	STData.WindBranchAdherences = SpeedTreeData.PrevWindBranchAdherences;
	STData.WindTurbulences = SpeedTreeData.PrevWindTurbulences;
	STData.WindLeaf1Ripple = SpeedTreeData.PrevWindLeaf1Ripple;
	STData.WindLeaf1Tumble = SpeedTreeData.PrevWindLeaf1Tumble;
	STData.WindLeaf1Twitch = SpeedTreeData.PrevWindLeaf1Twitch;
	STData.WindLeaf2Ripple = SpeedTreeData.PrevWindLeaf2Ripple;
	STData.WindLeaf2Tumble = SpeedTreeData.PrevWindLeaf2Tumble;
	STData.WindLeaf2Twitch = SpeedTreeData.PrevWindLeaf2Twitch;
	STData.WindFrondRipple = SpeedTreeData.PrevWindFrondRipple;
	STData.WindRollingBranch = SpeedTreeData.PrevWindRollingBranch;
	STData.WindRollingLeafAndDirection = SpeedTreeData.PrevWindRollingLeafAndDirection;
	STData.WindRollingNoise = SpeedTreeData.PrevWindRollingNoise;
	STData.WindAnimation = SpeedTreeData.PrevWindAnimation;
	return STData;
}

///////////////////////////////////////////////////////////////////////  
//  SpeedTree v7 Wind
//

#define SPEEDTREE_Z_UP

#define SPEEDTREE_WIND_TYPE_NONE 0
#define SPEEDTREE_WIND_TYPE_FASTEST 1
#define SPEEDTREE_WIND_TYPE_FAST 2
#define SPEEDTREE_WIND_TYPE_BETTER 3
#define SPEEDTREE_WIND_TYPE_BEST 4
#define SPEEDTREE_WIND_TYPE_PALM 5
#define SPEEDTREE_WIND_TYPE_BEST_PLUS 6

#define SPEEDTREE_GEOMETRY_TYPE_BRANCH 0
#define SPEEDTREE_GEOMETRY_TYPE_FROND 1
#define SPEEDTREE_GEOMETRY_TYPE_LEAF 2
#define SPEEDTREE_GEOMETRY_TYPE_FACINGLEAF 3
#define SPEEDTREE_GEOMETRY_TYPE_BILLBOARD 4

#define SPEEDTREE_LOD_TYPE_POP 0
#define SPEEDTREE_LOD_TYPE_SMOOTH 1


// SpeedTree texture coordinate setup
//
//		Branches			Fronds				Leaves				Billboards
//
// 0	Diffuse				Diffuse				Diffuse				Diffuse			
// 1	Lightmap UV			Lightmap UV			Lightmap UV			Lightmap UV	(same as diffuse)		
// 2	Branch Wind XY		Branch Wind XY		Branch Wind XY			
// 3	LOD XY				LOD XY				LOD XY				
// 4	LOD Z, Seam Amount	LOD Z, 0			LOD Z, Anchor X		
// 5	Detail UV			Frond Wind XY		Anchor YZ	
// 6	Seam UV				Frond Wind Z, 0		Leaf Wind XY
// 7	0					0					Leaf Wind Z, Leaf Group


///////////////////////////////////////////////////////////////////////  
//  UnpackNormalFromFloat

float3 UnpackNormalFromFloat(float fValue)
{
	float3 vDecodeKey = float3(16.0, 1.0, 0.0625);

	// decode into [0,1] range
	float3 vDecodedValue = frac(fValue / vDecodeKey);

	// move back into [-1,1] range & normalize
	return (vDecodedValue * 2.0 - 1.0);
}


///////////////////////////////////////////////////////////////////////  
//  CubicSmooth

float4 CubicSmooth(float4 vData)
{
	return vData * vData * (3.0 - 2.0 * vData);
}


///////////////////////////////////////////////////////////////////////  
//  TriangleWave

float4 TriangleWave(float4 vData)
{
	return abs((frac(vData + 0.5) * 2.0) - 1.0);
}


///////////////////////////////////////////////////////////////////////  
//  TrigApproximate

float4 TrigApproximate(float4 vData)
{
	return (CubicSmooth(TriangleWave(vData)) - 0.5) * 2.0;
}


///////////////////////////////////////////////////////////////////////  
//  RotationMatrix
//
//  Constructs an arbitrary axis rotation matrix

float3x3 RotationMatrix(float3 vAxis, float fAngle)
{
    // compute sin/cos of fAngle
    float2 vSinCos;
	#if COMPILER_GLSL || COMPILER_GLSL_ES3_1
		vSinCos.x = sin(fAngle);
		vSinCos.y = cos(fAngle);
	#else
		sincos(fAngle, vSinCos.x, vSinCos.y);
	#endif
    
	const float c = vSinCos.y;
	const float s = vSinCos.x;
	const float t = 1.0 - c;
	const float x = vAxis.x;
	const float y = vAxis.y;
	const float z = vAxis.z;
    
    return float3x3(t * x * x + c,		t * x * y - s * z,	t * x * z + s * y,
					t * x * y + s * z,	t * y * y + c,		t * y * z - s * x,
					t * x * z - s * y,	t * y * z + s * x,	t * z * z + c);
}


///////////////////////////////////////////////////////////////////////  
//  mul_float3x3_float3x3

float3x3 mul_float3x3_float3x3(float3x3 mMatrixA, float3x3 mMatrixB)
{
	return mul(mMatrixA, mMatrixB);
}


///////////////////////////////////////////////////////////////////////  
//  mul_float3x3_float3

float3 mul_float3x3_float3(float3x3 mMatrix, float3 vVector)
{
	return mul(mMatrix, vVector);
}


///////////////////////////////////////////////////////////////////////  
//  cross()'s parameters are backwards in GLSL

#define wind_cross(a, b) cross((a), (b))


///////////////////////////////////////////////////////////////////////  
//  Roll

float Roll(FSpeedTreeData STData,
	float fCurrent,
	float fMaxScale,
	float fMinScale,
	float fSpeed,
	float fRipple,
	float3 vPos,
	float fTime)
{
	float fWindAngle = dot(vPos, -STData.WindVector.xyz) * fRipple;
	float fAdjust = TrigApproximate(float4(fWindAngle + fTime * fSpeed, 0.0, 0.0, 0.0)).x;
	fAdjust = (fAdjust + 1.0) * 0.5;
	
	return lerp(fCurrent * fMinScale, fCurrent * fMaxScale, fAdjust);
}


///////////////////////////////////////////////////////////////////////  
//  Twitch

float Twitch(float3 vPos, float fAmount, float fSharpness, float fTime)
{
	const float c_fTwitchFudge = 0.87;
	float4 vOscillations = TrigApproximate(float4(fTime + (vPos.x + vPos.z), c_fTwitchFudge * fTime + vPos.y, 0.0, 0.0));

	//float fTwitch = sin(fFreq1 * fTime + (vPos.x + vPos.z)) * cos(fFreq2 * fTime + vPos.y);
	float fTwitch = vOscillations.x * vOscillations.y * vOscillations.y;
	fTwitch = (fTwitch + 1.0) * 0.5;
	
	return fAmount * pow(saturate(fTwitch), fSharpness);
}


///////////////////////////////////////////////////////////////////////  
//  Oscillate
//
//  This function computes an oscillation value and whip value if necessary.
//  Whip and oscillation are combined like this to minimize calls to
//  TrigApproximate( ) when possible.

float Oscillate(FSpeedTreeData STData,
	float3 vPos, 
	float fTime, 
	float fOffset, 
	float fWeight, 
	float fWhip, 
	bool bWhip, 
	bool bRoll, 
	bool bComplex, 
	float fTwitch, 
	float fTwitchFreqScale,
	inout float4 vOscillations)
{
	float fOscillation = 1.0;
	if (bComplex)
	{
		if (bWhip)
			vOscillations = TrigApproximate(float4(fTime + fOffset, fTime * fTwitchFreqScale + fOffset, fTwitchFreqScale * 0.5 * (fTime + fOffset), fTime + fOffset + (1.0 - fWeight)));
		else
			vOscillations = TrigApproximate(float4(fTime + fOffset, fTime * fTwitchFreqScale + fOffset, fTwitchFreqScale * 0.5 * (fTime + fOffset), 0.0));

		float fFineDetail = vOscillations.x;
		float fBroadDetail = vOscillations.y * vOscillations.z;
		
		float fTarget = 1.0;
		float fAmount = fBroadDetail;
		if (fBroadDetail < 0.0)
		{
			fTarget = -fTarget;
			fAmount = abs(fAmount);
		}

		fBroadDetail = lerp(fBroadDetail, fTarget, fAmount);
		fBroadDetail = lerp(fBroadDetail, fTarget, fAmount);

		fOscillation = fBroadDetail * fTwitch * (1.0 - STData.WindVector.w) + fFineDetail * (1.0 - fTwitch);

		if (bWhip)
			fOscillation *= 1.0 + (vOscillations.w * fWhip);
	}
	else
	{
		if (bWhip)
			vOscillations = TrigApproximate(float4(fTime + fOffset, fTime * 0.689 + fOffset, 0.0, fTime + fOffset + (1.0 - fWeight)));
		else
			vOscillations = TrigApproximate(float4(fTime + fOffset, fTime * 0.689 + fOffset, 0.0, 0.0));

		fOscillation = vOscillations.x + vOscillations.y * vOscillations.x;

		if (bWhip)
			fOscillation *= 1.0 + (vOscillations.w * fWhip);
	}

	//if (bRoll)
	//{
	// 	fOscillation = Roll(fOscillation, STData.WindRollingBranches.x, STData.WindRollingBranches.y, STData.WindRollingBranches.z, STData.WindRollingBranches.w, vPos.xyz, fTime + fOffset);
	//}

	return fOscillation;
}


///////////////////////////////////////////////////////////////////////  
//  Turbulence

float Turbulence(float fTime, float fOffset, float fGlobalTime, float fTurbulence)
{
	const float c_fTurbulenceFactor = 0.1;
	
	float4 vOscillations = TrigApproximate(float4(fTime * c_fTurbulenceFactor + fOffset, fGlobalTime * fTurbulence * c_fTurbulenceFactor + fOffset, 0.0, 0.0));

	return 1.0 - (vOscillations.x * vOscillations.y * vOscillations.x * vOscillations.y * fTurbulence);
}


///////////////////////////////////////////////////////////////////////  
//  GlobalWind
//
//  This function positions any tree geometry based on their untransformed 
//	position and 4 wind floats.

float3 GlobalWind(FSpeedTreeData STData, float3 vPos, float3 vInstancePos, bool bPreserveShape, bool bGlobalWind, bool bExtraBend, float3 ExtraBend)
{
	// WIND_LOD_GLOBAL may be on, but if the global wind effect (WIND_EFFECT_GLOBAL_WIND)
	// was disabled for the tree in the Modeler, we should skip it

	float fLength = 1.0;
	if (bPreserveShape)
		fLength = length(vPos.xyz);

	// compute how much the height contributes
	#ifdef SPEEDTREE_Z_UP
		float fAdjust = max(vPos.z - (1.0 / STData.WindGlobal.z) * 0.25, 0.0) * STData.WindGlobal.z;
	#else
		float fAdjust = max(vPos.y - (1.0 / STData.WindGlobal.z) * 0.25, 0.0) * STData.WindGlobal.z;
	#endif
	if (fAdjust != 0.0)
		fAdjust = pow(fAdjust, STData.WindGlobal.w);

	if (bGlobalWind)
	{
		// primary oscillation
		float4 vOscillations = TrigApproximate(float4(vInstancePos.x + STData.WindGlobal.x, vInstancePos.y + STData.WindGlobal.x * 0.8, 0.0, 0.0));
		float fOsc = vOscillations.x + (vOscillations.y * vOscillations.y);
		float fMoveAmount = STData.WindGlobal.y * fOsc;

		// move a minimum amount based on direction adherence
		float fMinMove = STData.WindBranchAdherences.x;
		if (STData.WindGlobal.z != 0.0) // not sure why I need this check
			fMinMove /= STData.WindGlobal.z;
		fMoveAmount += fMinMove;

		// adjust based on how high up the tree this vertex is
		fMoveAmount *= fAdjust;

		// xy component
		#ifdef SPEEDTREE_Z_UP
			vPos.xy += STData.WindVector.xy * fMoveAmount;
		#else
			vPos.xz += STData.WindVector.xz * fMoveAmount;
		#endif
	}

	if (bExtraBend)
	{
		#ifdef SPEEDTREE_Z_UP
			vPos.xy += ExtraBend.xy * fAdjust;
		#else
			vPos.xz += ExtraBend.xz * fAdjust;
		#endif
	}

	if (bPreserveShape)
		vPos.xyz = normalize(vPos.xyz) * fLength;


	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  SimpleBranchWind

float3 SimpleBranchWind(
	FSpeedTreeData STData,
	float3 vPos,
	float3 vInstancePos,
	float fWeight, 
	float fOffset, 
	float fTime, 
	float fDistance, 
	float fTwitch, 
	float fTwitchScale, 
	float fWhip, 
	bool bWhip, 
	bool bRoll, 
	bool bComplex)
{
	// turn the offset back into a nearly normalized vector
	float3 vWindVector = UnpackNormalFromFloat(fOffset);
	vWindVector = vWindVector * fWeight;
	
	// try to fudge time a bit so that instances aren't in sync
	fTime += vInstancePos.x + vInstancePos.y;

	// oscillate
	float4 vOscillations;
	float fOsc = Oscillate(STData, vPos, fTime, fOffset, fWeight, fWhip, bWhip, bRoll, bComplex, fTwitch, fTwitchScale, vOscillations);

	vPos.xyz += vWindVector * fOsc * fDistance;

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  DirectionalBranchWind

float3 DirectionalBranchWind(FSpeedTreeData STData,
	float3 vPos, 
	float3 vInstancePos,
	float fWeight, 
	float fOffset, 
	float fTime, 
	float fDistance, 
	float fTurbulence, 
	float fAdherence, 
	float fTwitch, 
	float fTwitchScale, 
	float fWhip, 
	bool bWhip, 
	bool bRoll, 
	bool bComplex, 
	bool bTurbulence)
{
	// turn the offset back into a nearly normalized vector
	float3 vWindVector = UnpackNormalFromFloat(fOffset);
	vWindVector = vWindVector * fWeight;
	
	// try to fudge time a bit so that instances aren't in sync
	fTime += vInstancePos.x + vInstancePos.y;
	
	// oscillate
	float4 vOscillations;
	float fOsc = Oscillate(STData, vPos, fTime, fOffset, fWeight, fWhip, bWhip, false, bComplex, fTwitch, fTwitchScale, vOscillations);

	vPos.xyz += vWindVector * fOsc * fDistance;

	// add in the direction, accounting for turbulence
	float fAdherenceScale = 1.0;
	if (bTurbulence)
		fAdherenceScale = Turbulence(fTime, fOffset, STData.WindAnimation.x, fTurbulence);

	if (bWhip)
		fAdherenceScale += vOscillations.w * STData.WindVector.w * fWhip;

	//if (bRoll)
	//	fAdherenceScale = Roll(fAdherenceScale, STData.WindRollingBranches.x, STData.WindRollingBranches.y, STData.WindRollingBranches.z, STData.WindRollingBranches.w, vPos.xyz, fTime + fOffset);

	vPos.xyz += STData.WindVector.xyz * fAdherence * fAdherenceScale * fWeight;

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  DirectionalBranchWindFrondStyle

float3 DirectionalBranchWindFrondStyle(FSpeedTreeData STData,
	float3 vPos, 
	float3 vInstancePos,
	float fWeight, 
	float fOffset, 
	float fTime, 
	float fDistance, 
	float fTurbulence, 
	float fAdherence, 
	float fTwitch, 
	float fTwitchScale, 
	float fWhip, 
	bool bWhip, 
	bool bRoll, 
	bool bComplex, 
	bool bTurbulence)
{
	// turn the offset back into a nearly normalized vector
	float3 vWindVector = UnpackNormalFromFloat(fOffset);
	vWindVector = vWindVector * fWeight;
	
	// try to fudge time a bit so that instances aren't in sync
	fTime += vInstancePos.x + vInstancePos.y;
	
	// oscillate
	float4 vOscillations;
	float fOsc = Oscillate(STData, vPos, fTime, fOffset, fWeight, fWhip, bWhip, false, bComplex, fTwitch, fTwitchScale, vOscillations);

	vPos.xyz += vWindVector * fOsc * fDistance;

	// add in the direction, accounting for turbulence
	float fAdherenceScale = 1.0;
	if (bTurbulence)
		fAdherenceScale = Turbulence(fTime, fOffset, STData.WindAnimation.x, fTurbulence);

	//if (bRoll)
	//	fAdherenceScale = Roll(fAdherenceScale, STData.WindRollingBranches.x, STData.WindRollingBranches.y, STData.WindRollingBranches.z, STData.WindRollingBranches.w, vPos.xyz, fTime + fOffset);

	if (bWhip)
		fAdherenceScale += vOscillations.w * STData.WindVector.w * fWhip;

	float3 vWindAdherenceVector = STData.WindBranchAnchor.xyz - vPos.xyz;
	vPos.xyz += vWindAdherenceVector * fAdherence * fAdherenceScale * fWeight;

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  BranchWind

float3 BranchWind(FSpeedTreeData STData, float3 vPos, float3 vInstancePos, float4 vWindData, int WindType)
{
	if (WindType == SPEEDTREE_WIND_TYPE_PALM)
	{
		vPos = DirectionalBranchWindFrondStyle(STData, vPos, vInstancePos, vWindData.x, vWindData.y, STData.WindBranch.x, STData.WindBranch.y, STData.WindTurbulences.x, STData.WindBranchAdherences.y, STData.WindBranchTwitch.x, STData.WindBranchTwitch.y, STData.WindBranchWhip.x, true, false, true, true);
	}
	else
	{
		vPos = SimpleBranchWind(STData, vPos, vInstancePos, vWindData.x, vWindData.y, STData.WindBranch.x, STData.WindBranch.y, STData.WindBranchTwitch.x, STData.WindBranchTwitch.y, STData.WindBranchWhip.x, false, false, true);
	}

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  LeafRipple

float3 LeafRipple(float3 vPos, 
				  inout float3 vDirection, 
				  float fScale, 
				  float fPackedRippleDir, 
				  float fTime, 
				  float fAmount, 
				  bool bDirectional,
				  float fTrigOffset)
{
	// compute how much to move
	float4 vInput = float4(fTime + fTrigOffset, 0.0, 0.0, 0.0);
 	float fMoveAmount = fAmount * TrigApproximate(vInput).x;

	if (bDirectional)
	{
		vPos.xyz += vDirection.xyz * fMoveAmount * fScale;
	}
	else
	{
		float3 vRippleDir = UnpackNormalFromFloat(fPackedRippleDir);
		vPos.xyz += vRippleDir * fMoveAmount * fScale;
	}

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  LeafTumble
      
float3 LeafTumble(FSpeedTreeData STData,
	float3 vPos, 
	inout float3 vDirection, 
	float fScale, 
	float3 vAnchor, 
	float3 vGrowthDir, 
	float fTrigOffset, 
	float fTime, 
	float fFlip, 
	float fTwist, 
	float fAdherence, 
	float3 vTwitch, 
	float4 vRoll,
	int WindType,
	bool bTwitch, 
	bool bRoll)
{
	// compute all oscillations up front
 	float3 vFracs = frac((vAnchor + fTrigOffset) * 30.3);
 	float fOffset = vFracs.x + vFracs.y + vFracs.z;
	float4 vOscillations = TrigApproximate(float4(fTime + fOffset, fTime * 0.75 - fOffset, fTime * 0.01 + fOffset, fTime * 1.0 + fOffset));
	
	// move to the origin and get the growth direction
	float3 vOriginPos = vPos.xyz - vAnchor;
	float fLength = length(vOriginPos);

	// twist
 	float fOsc = vOscillations.x + vOscillations.y * vOscillations.y;
 	float3x3 matTumble = RotationMatrix(vGrowthDir, fScale * fTwist * fOsc);

	if (WindType >= SPEEDTREE_WIND_TYPE_BEST_PLUS)
	{
		// with wind
		float3 vAxis = wind_cross(vGrowthDir, STData.WindVector.xyz);
		float fDot = clamp(dot(STData.WindVector.xyz, vGrowthDir), -1.0, 1.0);
#ifdef SPEEDTREE_Z_UP
		vAxis.z += fDot;
#else
		vAxis.y += fDot;
#endif
		vAxis = normalize(vAxis);

		float fAngle = acos(fDot);

		float fAdherenceScale = 1.0;
		//if (bRoll)
		//	fAdherenceScale = Roll(fAdherenceScale, vRoll.x, vRoll.y, vRoll.z, vRoll.w, vAnchor.xyz, fTime/* + fOffset*/);

		fOsc = vOscillations.y - vOscillations.x * vOscillations.x;

		float fTwitch = 0.0;
		if (bTwitch)
			fTwitch = Twitch(vAnchor.xyz, vTwitch.x, vTwitch.y, vTwitch.z + fOffset);


		matTumble = mul_float3x3_float3x3(matTumble, RotationMatrix(vAxis, fScale * (fAngle * fAdherence * fAdherenceScale + fOsc * fFlip + fTwitch)));
		//	matTumble = mul_float3x3_float3x3(matTumble, RotationMatrix(vAxis, fAngle));
	}
 
 	vDirection = mul_float3x3_float3(matTumble, vDirection);
	vOriginPos = mul_float3x3_float3(matTumble, vOriginPos);

	vOriginPos = normalize(vOriginPos) * fLength;

	return (vOriginPos + vAnchor);
}


///////////////////////////////////////////////////////////////////////  
//  RippleFrondOneSided

float3 RippleFrondOneSided(FSpeedTreeData STData,
	float3 vPos, 
	inout float3 vDirection, 
	float fU, 
	float fV, 
	float fRippleScale
#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
	, float3 vBinormal
	, float3 vTangent
#endif
						   )
{
	float fOffset = 0.0;
	if (fU < 0.5)
		fOffset = 0.75;

	float4 vOscillations = TrigApproximate(float4((STData.WindFrondRipple.x + fV) * STData.WindFrondRipple.z + fOffset, 0.0, 0.0, 0.0));

	float fAmount = fRippleScale * vOscillations.x * STData.WindFrondRipple.y;
	float3 vOffset = fAmount * vDirection;
	vPos.xyz += vOffset;

	#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
		vTangent.xyz = normalize(vTangent.xyz + vOffset * STData.WindFrondRipple.w);
		float3 vNewNormal = normalize(wind_cross(vBinormal.xyz, vTangent.xyz));
		if (dot(vNewNormal, vDirection.xyz) < 0.0)
			vNewNormal = -vNewNormal;
		vDirection.xyz = vNewNormal;
	#endif

	return vPos;
}

///////////////////////////////////////////////////////////////////////  
//  RippleFrondTwoSided

float3 RippleFrondTwoSided(FSpeedTreeData STData,
	float3 vPos, 
	inout float3 vDirection, 
	float fU, 
	float fLengthPercent, 
	float fPackedRippleDir, 
	float fRippleScale
#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
	, float3 vBinormal
	, float3 vTangent
#endif
						   )
{
	float4 vOscillations = TrigApproximate(float4(STData.WindFrondRipple.x * fLengthPercent * STData.WindFrondRipple.z, 0.0, 0.0, 0.0));

	float3 vRippleDir = UnpackNormalFromFloat(fPackedRippleDir);

	float fAmount = fRippleScale * vOscillations.x * STData.WindFrondRipple.y;
	float3 vOffset = fAmount * vRippleDir;

	vPos.xyz += vOffset;

	#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
		vTangent.xyz = normalize(vTangent.xyz + vOffset * STData.WindFrondRipple.w);
		float3 vNewNormal = normalize(wind_cross(vBinormal.xyz, vTangent.xyz));
		if (dot(vNewNormal, vDirection.xyz) < 0.0)
			vNewNormal = -vNewNormal;
		vDirection.xyz = vNewNormal;
	#endif

	return vPos;
}


///////////////////////////////////////////////////////////////////////  
//  RippleFrond

float3 RippleFrond(FSpeedTreeData STData,
	float3 vPos, 
	inout float3 vDirection,
	float fU, 
	float fV, 
	float fPackedRippleDir, 
	float fRippleScale, 
	float fLenghtPercent
#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
	, float3 vBinormal
	, float3 vTangent
#endif
				   )
{
	return RippleFrondOneSided(STData, vPos, 
								vDirection, 
								fU, 
								fV, 
								fRippleScale
#ifdef WIND_EFFECT_FROND_RIPPLE_ADJUST_LIGHTING
								, vBinormal
								, vTangent
#endif
								);
}




///////////////////////////////////////////////////////////////////////  
//  LeafWind
//  Optimized (for instruction count) version for UE4. Assumes leaf 1 and 2 have the same options

float3 LeafWind(FSpeedTreeData STData,
	bool bLeaf2,
	float3 vPos, 
	inout float3 vDirection, 
	float fScale, 
	float3 vAnchor, 
	float fPackedGrowthDir, 
	float fPackedRippleDir,
	float fRippleTrigOffset,
	int WindType)
{
	
	if (WindType >= SPEEDTREE_WIND_TYPE_BEST_PLUS)
	{
		vPos = LeafRipple(vPos, vDirection, fScale, fPackedRippleDir,
			(bLeaf2 ? STData.WindLeaf2Ripple.x : STData.WindLeaf1Ripple.x),
			(bLeaf2 ? STData.WindLeaf2Ripple.y : STData.WindLeaf1Ripple.y),
			false, fRippleTrigOffset);
	}


	if (WindType >= SPEEDTREE_WIND_TYPE_BEST)
	{
		float3 vGrowthDir = UnpackNormalFromFloat(fPackedGrowthDir);
		vPos = LeafTumble(STData, vPos, vDirection, fScale, vAnchor, vGrowthDir, fPackedGrowthDir,
						  (bLeaf2 ? STData.WindLeaf2Tumble.x : STData.WindLeaf1Tumble.x), 
						  (bLeaf2 ? STData.WindLeaf2Tumble.y : STData.WindLeaf1Tumble.y), 
						  (bLeaf2 ? STData.WindLeaf2Tumble.z : STData.WindLeaf1Tumble.z), 
						  (bLeaf2 ? STData.WindLeaf2Tumble.w : STData.WindLeaf1Tumble.w),
						  (bLeaf2 ? STData.WindLeaf2Twitch.xyz : STData.WindLeaf1Twitch.xyz), 
						  0.0f,
						  WindType,
						  (bLeaf2 ? true : true), 
						  (bLeaf2 ? true : true));
	}

	return vPos;
}
