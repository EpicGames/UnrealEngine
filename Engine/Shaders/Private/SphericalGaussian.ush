// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// G( v; u,L,a ) = a * exp( L * (dot(u,v) - 1) )

struct FSphericalGaussian
{
	float3	Axis;		// u
	float	Sharpness;	// L
	float	Amplitude;	// a
};

float Evaluate( FSphericalGaussian G, float3 Direction )
{
	// G( v; u,L,a ) = a * exp( L * (dot(u,v) - 1) )

	return G.Amplitude * exp( G.Sharpness * (dot( G.Axis, Direction ) - 1) );
}

// Integral over the sphere
float Integral( FSphericalGaussian G )
{
	// integral G  = 2*PI * a/L * ( 1 - exp(-2*L) )
	// integral G ~= 2*PI * a/L, L > 2

	return (2*PI) * G.Amplitude / G.Sharpness * ( 1 - exp( -2 * G.Sharpness ) );
}

// G / integral G
FSphericalGaussian Normalize( FSphericalGaussian G )
{
	// G / integral G  = G( v; u, L, L / ( 2*PI * ( 1 - exp(-2*L) ) ) )
	// G / integral G ~= G( v; u, L, L / (2*PI) ), L > 2

	G.Amplitude = G.Sharpness / ( (2*PI) - (2*PI) * exp( -2 * G.Sharpness ) );
	return G;
}

// G0 * G1
FSphericalGaussian Mul( FSphericalGaussian G0, FSphericalGaussian G1 )
{
	// um = L0 * u0 + L1 * u1
	// G0 * G1 = G( v; um / |um|, |um|, a0*a1 * exp( |um| - L0 - L1 ) )

	float  Lm = G0.Sharpness + G1.Sharpness;
	float3 um = G0.Sharpness * G0.Axis + G1.Sharpness * G1.Axis;
	float  umLength = length(um);

	FSphericalGaussian G =
	{
		um / umLength,
		umLength,
		G0.Amplitude * G1.Amplitude * exp( umLength - Lm )
	};

	return G;
}

// integral G0 * G1
float Dot( FSphericalGaussian G0, FSphericalGaussian G1 )
{
	// integral G0 * G1  = 4*PI * a0 * a1 * exp( -L0 - L1 ) * sinh( |um| ) / |um|
	// integral G0 * G1  = 2*PI * a0 * a1 * ( exp( |um| - L0 - L1 ) - exp( -|um| - L0 - L1 ) ) / |um|
	// integral G0 * G1 ~= 2*PI * a0 * a1 * exp( |um| - L0 - L1 ) / |um|
	
	float  Lm = G0.Sharpness + G1.Sharpness;
	float3 um = G0.Sharpness * G0.Axis + G1.Sharpness * G1.Axis;
	float  umLength = length(um);

	//return 2*PI * G0.a * G1.a * ( exp( umLength - G0.L - G1.L ) ) / umLength;
	//return 2*PI * G0.Amplitude * G1.Amplitude * ( exp( umLength - Lm ) - exp( -umLength - Lm ) ) / umLength;
	return (2*PI) * G0.Amplitude * G1.Amplitude * exp( umLength - Lm ) * ( 1 - exp( -2 * umLength ) ) / umLength;
}

// [ Iwasaki 2012, "Interactive Bi-scale Editing of Highly Glossy Materials" ]
FSphericalGaussian Convolve( FSphericalGaussian G0, FSphericalGaussian G1 )
{
	FSphericalGaussian G =
	{
		G0.Axis,
		( G0.Sharpness * G1.Sharpness ) / ( G0.Sharpness + G1.Sharpness ),
		(2*PI) * ( G0.Amplitude * G1.Amplitude ) / ( G0.Sharpness + G1.Sharpness )
	};

	return G;
}

// approximation using von Mises-Fisher
FSphericalGaussian ToSphericalGaussian( float3 r, float Value )
{
	// L = |r| * ( 3 - |r|^2 ) / ( 1 - |r|^2 )

	FSphericalGaussian G;

	float LengthR2 = dot( r, r );
	float InvLengthR = rsqrt( LengthR2 );
	float LengthR = LengthR2 * InvLengthR;

	G.Axis = r * InvLengthR;
	G.Sharpness = LengthR * ( 3 - LengthR2 ) / ( 1 - min( LengthR2, 0.9999 ) );
	G.Amplitude = Value * G.Sharpness / ( (2*PI) - (2*PI) * exp( -2 * G.Sharpness ) );
	//G.u = Value * G.L * (0.5/PI);

	return G;
}

FSphericalGaussian Add( FSphericalGaussian G0, FSphericalGaussian G1 )
{
	// r  = ( 1 / tanh(L) - 1/L ) * u
	// r  = ( ( 1 + exp( -2*L ) ) / ( 1 - exp( -2*L ) ) - 1/L ) * u
	// r ~= ( 1 - 1/L + 2 * exp( -2*L ) ) * u, L > 2

	float exp2L0 = exp( -2 * G0.Sharpness );
	float exp2L1 = exp( -2 * G1.Sharpness );

	float3 r0 = ( (1 + exp2L0) / (1 - exp2L0) - rcp( G0.Sharpness ) ) * G0.Axis;
	float3 r1 = ( (1 + exp2L1) / (1 - exp2L1) - rcp( G1.Sharpness ) ) * G1.Axis;
	float w0 = Integral( G0 );
	float w1 = Integral( G1 );

	float3 r = ( r0*w0 + r1*w1 ) / (w0 + w1);
	float w = w0 + w1;

	return ToSphericalGaussian( r, w );
}

// Angle from axis of cone. Half subtended angle.
float GetConeAngle( FSphericalGaussian G )
{
	// cone angle ~= PI - PI*|r|
	// r = ( 1 / tanh(k) - 1/k ) * u ~= ( 1 - 1/k + 2 * exp( -2*k ) ) * u, k > 2
	// ConeAngle ~= sqrt( 2/L )

	return sqrt( 2 / G.Sharpness );
}

// Inner product with cosine lobe
// Assumes G is normalized
float DotCosineLobe( FSphericalGaussian G, float3 N )
{
	const float muDotN = dot( G.Axis, N );

	const float c0 = 0.36;
	const float c1 = 0.25 / c0;

	float eml  = exp( -G.Sharpness );
	float em2l = eml * eml;
	float rl   = rcp( G.Sharpness );
 
	float scale = 1.0f + 2.0f * em2l - rl;
	float bias  = (eml - em2l) * rl - em2l;

	float x = sqrt( 1.0 - scale );
	float x0 = c0 * muDotN;
	float x1 = c1 * x;

	float n = x0 + x1;
	float y = ( abs( x0 ) <= x1 ) ? n*n / x : saturate( muDotN );

	return scale * y + bias;
}

// [ Wang et al. 2009, "All-Frequency Rendering of Dynamic, Spatially-Varying Reflectance" ]
FSphericalGaussian ClampedCosine_ToSphericalGaussian( float3 Normal )
{
	FSphericalGaussian G;
	
	G.Axis = Normal;
	G.Sharpness = 2.133;
	G.Amplitude = 1.17;

	// Integrate to PI
	//G.Sharpness = 2.3;
	//G.Amplitude = 0.5 * 2.3 / ( 1 - exp( -2 * 2.3 ) );

	return G;
}

FSphericalGaussian Hemisphere_ToSphericalGaussian( float3 Normal )
{
	FSphericalGaussian G;
	
	G.Axis = Normal;
	G.Sharpness = 0.81;
	G.Amplitude = 0.81 / ( 1 - exp( -2 * 0.81 ) );

	return G;
}

// Bent normal is normalized. AO is [0,1]. Both are cosine weighted.
FSphericalGaussian BentNormalAO_ToSphericalGaussian( float3 BentNormal, float AO )
{
	// ConeAngle ~= sqrt( 2/L )
	// L ~= 2/ConeAngle^2

	FSphericalGaussian G;
	
	G.Axis = BentNormal;

#if 1
	// Cosine weighted integration of spherical cap
	// PI * SinAlpha^2
	// L ~= 2 / Pow2( acos( sqrt(1- AO) ) );

	// Approximation (no acos)
	G.Sharpness = ( 0.75 + 1.25 * sqrt( 1 - AO ) ) / AO;
#else
	// Solid angle of cone = 2 * PI * (1 - CosTheta)
	// Solid angle of cone = 2*PI * AO
	// AO = 1 - cos( ConeAngle )
	// L ~= 2 / Pow2( acos( 1- AO ) );
	
	// Approximation (no acos)
	G.Sharpness = ( 1 - 0.19 * AO ) / AO;
#endif
	
	// AO=1 integrates to 2pi
	const float HemisphereSharpness = 0.81;
	G.Amplitude = HemisphereSharpness / ( 1 - exp( -2 * HemisphereSharpness ) );

	return G;
}

/*
static const float C[5] =
{
	0.5 / sqrt(PI),
	0.5 * sqrt(3/PI),
};

struct FSphericalHarmonics
{
	float	SH[4];
};

FSphericalHarmonics Add( float3 Direction, float Value )
{
}*/

struct FAnisoSphericalGaussian
{
	float3	AxisX;
	float3	AxisY;
	float3	AxisZ;
	float	SharpnessX;
	float	SharpnessY;
	float	Amplitude;
};

float Evaluate( FAnisoSphericalGaussian ASG, float3 Direction )
{
	float L = ASG.SharpnessX * Pow2( dot( Direction, ASG.AxisX ) );
	float u = ASG.SharpnessY * Pow2( dot( Direction, ASG.AxisY ) );
	return ASG.Amplitude * saturate( dot( Direction, ASG.AxisZ ) ) * exp( -L - u );
}

float Dot( FAnisoSphericalGaussian ASG, FSphericalGaussian SG )
{
	// ASG( v; u,nu,a ) = a * exp( 2 * nu * (dot(u,v) - 1) )

	float nu = SG.Sharpness * 0.5;

	ASG.Amplitude *= SG.Amplitude;
	ASG.Amplitude *= PI * rsqrt( (nu + ASG.SharpnessX) * (nu + ASG.SharpnessY) );
	ASG.SharpnessX = (nu * ASG.SharpnessX) / (nu + ASG.SharpnessX);
	ASG.SharpnessY = (nu * ASG.SharpnessY) / (nu + ASG.SharpnessY);

	return Evaluate( ASG, SG.Axis );
}