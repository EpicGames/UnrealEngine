// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/SplineMeshShaderParams.h"
#include "ShaderPrint.ush"
#include "SceneData.ush"

#ifndef USE_SPLINE_MESH_SCENE_RESOURCES
	#define USE_SPLINE_MESH_SCENE_RESOURCES 0
#endif

// Methods for calculating deformed spline mesh bounds
#define SPLINE_MESH_DEFORM_BOUNDS_METHOD_SPHERES	0
#define SPLINE_MESH_DEFORM_BOUNDS_METHOD_QUADS		1

#ifndef SPLINE_MESH_DEFORM_BOUNDS_METHOD
	#define SPLINE_MESH_DEFORM_BOUNDS_METHOD SPLINE_MESH_DEFORM_BOUNDS_METHOD_SPHERES
#endif

FSplineMeshShaderParams SplineMeshLoadParamsFromInstancePayload(uint PayloadOffset)
{
	const float4 SplineParams[SPLINE_MESH_PARAMS_FLOAT4_SIZE] =
	{
		LoadInstancePayloadDataElement(PayloadOffset + 0),
		LoadInstancePayloadDataElement(PayloadOffset + 1),
		LoadInstancePayloadDataElement(PayloadOffset + 2),
		LoadInstancePayloadDataElement(PayloadOffset + 3),
		LoadInstancePayloadDataElement(PayloadOffset + 4),
		LoadInstancePayloadDataElement(PayloadOffset + 5),
		LoadInstancePayloadDataElement(PayloadOffset + 6)
	};

	return UnpackSplineMeshParams(SplineParams);
}

FSplineMeshShaderParams SplineMeshLoadParamsFromInstancePayload(FInstanceSceneData InstanceData)
{
	checkSlow(InstanceData.PayloadExtensionOffset != INVALID_INSTANCE_PAYLOAD_OFFSET);
	return SplineMeshLoadParamsFromInstancePayload(InstanceData.PayloadExtensionOffset);
}

/** Calculate normalized distance along the spline based on local mesh position */
half SplineMeshCalcSplineDistance(FSplineMeshShaderParams Params, float3 LocalPos)
{
	float ZPos = dot(LocalPos, Params.MeshDir);
	return half(ZPos * Params.MeshZScale + Params.MeshZOffset);
}

/** Evaluates the position on the spline based on normalized distance along the spline */
float3 SplineMeshEvalSplinePos(FSplineMeshShaderParams Params, half SplineDist)
{
	float A = SplineDist;
	float A2 = A * A;
	float A3 = A * A2;

	return (((2*A3)-(3*A2)+1) * Params.StartPos) +
		   ((A3-(2*A2)+A) * Params.StartTangent) +
		   ((A3-A2) * Params.EndTangent) +
		   (((-2*A3)+(3*A2)) * Params.EndPos);
}

/** Evaluates the normalized tangent direction of the spline at a specified normalized distance along the spline */
half3 SplineMeshEvalSplineDir(FSplineMeshShaderParams Params, half SplineDist)
{
	float3 C = (6*Params.StartPos) + (3*Params.StartTangent) + (3*Params.EndTangent) - (6*Params.EndPos);
	float3 D = (-6*Params.StartPos) - (4*Params.StartTangent) - (2*Params.EndTangent) + (6*Params.EndPos);
	float3 E = Params.StartTangent;

	float A = SplineDist;
	float A2 = A * A;

	return half3(normalize((C * A2) + (D * A) + E));
}

/** Evaluates the scale of a slice of the spline at the specified normalized distance */
half2 SplineMeshEvalSliceScale(FSplineMeshShaderParams Params, half SplineDist)
{
	half HermiteAlpha = Params.bSmoothInterpRollScale ? smoothstep(half(0.0), half(1.0), SplineDist) : SplineDist;
	return lerp(Params.StartScale, Params.EndScale, HermiteAlpha);
}

/** Evaluates the roll of a slice of the spline at the specified normalized distance */
half SplineMeshEvalSliceRoll(FSplineMeshShaderParams Params, half SplineDist)
{
	half HermiteAlpha = Params.bSmoothInterpRollScale ? smoothstep(half(0.0), half(1.0), SplineDist) : SplineDist;
	return lerp(Params.StartRoll, Params.EndRoll, HermiteAlpha);
}

/** Evaluates the offset of a slice of the spline at the specified normalized distance */
float2 SplineMeshEvalSliceOffset(FSplineMeshShaderParams Params, half SplineDist)
{
	float HermiteAlpha = Params.bSmoothInterpRollScale ? smoothstep(0.0, 1.0, SplineDist) : SplineDist;
	return lerp(Params.StartOffset, Params.EndOffset, HermiteAlpha);
}

/**
 * Data of a single cross-sectional slice of a spline mesh, used to compose various transforms required for
 * spline mesh deformation.
 */
struct FSplineMeshSlice
{	
	float3 Pos;
	FQuat Quat;
	half3x3 Rot;
	half2 ScaleXY;
};

#if USE_SPLINE_MESH_SCENE_RESOURCES

/** Evaluates all parameters of a slice of the spline mesh at the specified normalized distance along the spline by sampling scene textures. */
FSplineMeshSlice SplineMeshCalcSlice(FSplineMeshShaderParams Params, half SplineDist)
{
	// Sample the slice position and rotation from scene textures
	const half SplineLastTexel = (SPLINE_MESH_TEXEL_WIDTH - 1);
	const half SplineDistTexels = clamp(SplineDist * Params.SplineDistToTexelScale + Params.SplineDistToTexelOffset, 0, SplineLastTexel);
	const float2 TexelSizeUV = Scene.SplineMesh.SplineTextureInvExtent;
	const float2 UVOffset = (Params.TextureCoord + (float2)0.5f) * TexelSizeUV;
	const float2 UV = UVOffset + float2(SplineDistTexels * TexelSizeUV.x, 0);
	const float3 Pos = Scene.SplineMesh.SplinePosTexture.SampleLevel(Scene.SplineMesh.SplineSampler, UV, 0).xyz;

	const half Slerp = frac(SplineDistTexels);
	const uint2 TC0 = Params.TextureCoord + uint2(SplineDistTexels, 0);
	const uint2 TC1 = Params.TextureCoord + uint2(min(SplineDistTexels + 1.0f, SplineLastTexel), 0);
	const FQuat Q0 = FQuat(Scene.SplineMesh.SplineRotTexture.Load(uint3(TC0, 0)));
	const FQuat Q1 = FQuat(Scene.SplineMesh.SplineRotTexture.Load(uint3(TC1, 0)));

	FSplineMeshSlice Output;
	Output.Pos = Pos;
	Output.Quat = QuatSlerp(Q0, Q1, Slerp);
	Output.Rot = QuatToMatrix(Output.Quat);
	Output.ScaleXY = SplineMeshEvalSliceScale(Params, SplineDist);

	return Output;
}

#else // !USE_SPLINE_MESH_SCENE_RESOURCES

/** Numerically evaluates all parameters of a slice of the spline mesh at the specified normalized distance along the spline. */
FSplineMeshSlice SplineMeshCalcSlice(FSplineMeshShaderParams Params, half SplineDist)
{
	// Evaluate all spline mesh slice parameters at given distance
	const float3 SplinePos = SplineMeshEvalSplinePos(Params, SplineDist);
	const half3 SplineDir = SplineMeshEvalSplineDir(Params, SplineDist);
	const half Roll = SplineMeshEvalSliceRoll(Params, SplineDist);
	const float2 Offset = SplineMeshEvalSliceOffset(Params, SplineDist);
	const half2 Scale = SplineMeshEvalSliceScale(Params, SplineDist);

	// Find base frenet frame
	const half3 BaseXVec = normalize( cross(Params.SplineUpDir, SplineDir) );
	const half3 BaseYVec = cross(SplineDir, BaseXVec);

	// Apply roll to frame around spline
	half SinAng, CosAng;
	sincos(Roll, SinAng, CosAng);
	const half3 XVec = (CosAng * BaseXVec) - (SinAng * BaseYVec);
	const half3 YVec = (CosAng * BaseYVec) + (SinAng * BaseXVec);

	FSplineMeshSlice Output;
	Output.Pos = SplinePos + Offset.x * BaseXVec + Offset.y * BaseYVec;
	Output.Rot = half3x3(SplineDir, XVec, YVec);
	Output.Quat = QuatFromMatrix(Output.Rot);
	Output.ScaleXY = Scale;

	return Output;
}

#endif // USE_SPLINE_MESH_SCENE_RESOURCES

/** Calculate full transform that defines frame along spline, given the pre-calculated slice data. */
float4x3 SplineMeshCalcSliceTransform(FSplineMeshShaderParams Params, FSplineMeshSlice Slice)
{
	// Apply scale to the X/Y vector directions
	const float3 XVec = Slice.ScaleXY.x * float3(Slice.Rot[1]);
	const float3 YVec = Slice.ScaleXY.y * float3(Slice.Rot[2]);

	// Build overall transform
	float3x3 SliceTransform3 = mul(transpose(float3x3(Params.MeshDir, Params.MeshX, Params.MeshY)), float3x3(float3(0,0,0), XVec, YVec));
	float4x3 SliceTransform = float4x3(SliceTransform3[0], SliceTransform3[1], SliceTransform3[2], Slice.Pos);

	return SliceTransform;
}

/** Calculate full transform that defines frame along spline, given the normalized distance along the spline. */
float4x3 SplineMeshCalcSliceTransform(FSplineMeshShaderParams Params, half SplineDist)
{
	// Find the center, orientation, and scale of the slice at this point along the spline
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(Params, SplineDist);
	return SplineMeshCalcSliceTransform(Params, Slice);
}

/** Calculate full transform that defines frame along spline, given the local position of a vertex. */
float4x3 SplineMeshCalcSliceTransformFromLocalPos(FSplineMeshShaderParams Params, float3 LocalPos)
{
	return SplineMeshCalcSliceTransform(Params, SplineMeshCalcSplineDistance(Params, LocalPos));
}

/** Calculate rotation matrix that defines frame along spline, given the pre-calculated slice data. */
half3x3 SplineMeshCalcSliceRot(FSplineMeshShaderParams Params, FSplineMeshSlice Slice)
{
	// Flip X or Y direction when negative scale
	const half3 XVec = half(Slice.ScaleXY.x >= 0.0f ? 1.0f : -1.0f) * Slice.Rot[1];
	const half3 YVec = half(Slice.ScaleXY.y >= 0.0f ? 1.0f : -1.0f) * Slice.Rot[2];

	// Build rotation transform
	const half3x3 SliceTransform = mul(transpose(half3x3(Params.MeshDir, Params.MeshX, Params.MeshY)), half3x3(Slice.Rot[0], XVec, YVec));

	return SliceTransform;
}

/** Calculate rotation matrix that defines frame along spline, given the normalized distance along the spline. */
half3x3 SplineMeshCalcSliceRot(FSplineMeshShaderParams Params, half SplineDist)
{
	// Find the center, orientation, and scale of the slice at this point along the spline
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(Params, SplineDist);
	return SplineMeshCalcSliceRot(Params, Slice);
}

/** Calculate rotation matrix that defines frame along spline, given the local position of a vertex. */
half3x3 SplineMeshCalcSliceRotFromLocalPos(FSplineMeshShaderParams Params, float3 LocalPos)
{
	return SplineMeshCalcSliceRot(Params, SplineMeshCalcSplineDistance(Params, LocalPos));
}

/** Deforms a local-space position along the spline, given its pre-calculated spline distance */
float3 SplineMeshDeformLocalPos(FSplineMeshShaderParams Params, half SplineDist, float3 LocalPos)
{
	const float4x3 SliceTransform = SplineMeshCalcSliceTransform(Params, SplineDist);
	return mul(float4(LocalPos, 1), SliceTransform).xyz;
}

/** Deforms a local-space position along the spline. */
float3 SplineMeshDeformLocalPos(FSplineMeshShaderParams Params, float3 LocalPos)
{
	const float4x3 SliceTransform = SplineMeshCalcSliceTransformFromLocalPos(Params, LocalPos);
	return mul(float4(LocalPos, 1), SliceTransform).xyz;
}

/** Deforms a local-space normal at the specified distance along the spline. */
float3 SplineMeshDeformLocalNormal(FSplineMeshShaderParams Params, half SplineDist, half3 LocalNormal)
{
	const half3x3 SliceRot = SplineMeshCalcSliceRot(Params, SplineDist);
	return mul(LocalNormal, SliceRot);
}

/** Deforms local-space position and normal along a spline, and retrieves the spline length */
half SplineMeshDeformLocalPosNormalTangent(FSplineMeshShaderParams Params, inout float3 Pos, inout half3 Normal, inout half3 Tangent)
{
	const half SplineDist = SplineMeshCalcSplineDistance(Params, Pos);
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(Params, SplineDist);
	
	Pos = mul(float4(Pos, 1), SplineMeshCalcSliceTransform(Params, Slice)).xyz;

	const half3x3 SliceRot = SplineMeshCalcSliceRot(Params, Slice);
	Normal = mul(Normal, SliceRot);
	Tangent = mul(Tangent, SliceRot);
	
	return SplineDist;
}

struct FSplineMeshDeformBoundsContext
{
	FSplineMeshShaderParams Params;
	float3 MeshBoundsCenter;
	float3 MeshBoundsExtent;
	float3 MeshMinBounds;
	float3 MeshMaxBounds;
	float3 DeformedMinBounds;
	float3 DeformedMaxBounds;
	float3 LastSlicePos;
	float CurSplineLength;
	float MaxScaleXY;
	FShaderPrintContext ShaderPrint;
	float4x4 LocalToTWS;
};

struct FSplineMeshDeformedLocalBounds
{
	float3 BoundsCenter;
	float3 BoundsExtent;
	float MaxDeformScale;
};

FSplineMeshDeformBoundsContext SplineMeshInitializeDeformBoundsContext(
	FSplineMeshShaderParams Params,
	float3 MeshBoundsCenter,
	float3 MeshBoundsExtent,
	FShaderPrintContext ShaderPrint,
	float4x4 LocalToTWS
)
{
	FSplineMeshDeformBoundsContext Result;
	Result.Params				= Params;
	Result.MeshBoundsCenter		= MeshBoundsCenter;
	Result.MeshBoundsExtent		= MeshBoundsExtent;
	Result.MeshMinBounds		= MeshBoundsCenter - MeshBoundsExtent;
	Result.MeshMaxBounds		= MeshBoundsCenter + MeshBoundsExtent;
	Result.DeformedMinBounds	= (float3)POSITIVE_INFINITY;
	Result.DeformedMaxBounds	= (float3)NEGATIVE_INFINITY;
	Result.CurSplineLength		= -1.0f; // Negative to mark as not started
	Result.MaxScaleXY			= 0.0f;
	Result.ShaderPrint			= ShaderPrint;
	Result.LocalToTWS			= LocalToTWS;

	return Result;
}

FSplineMeshDeformBoundsContext SplineMeshInitializeDeformBoundsContext(
	FSplineMeshShaderParams Params,
	float3 MeshBoundsCenter,
	float3 MeshBoundsExtent
)
{
	FSplineMeshDeformBoundsContext Result;
	Result.Params				= Params;
	Result.MeshBoundsCenter		= MeshBoundsCenter;
	Result.MeshBoundsExtent		= MeshBoundsExtent;
	Result.MeshMinBounds		= MeshBoundsCenter - MeshBoundsExtent;
	Result.MeshMaxBounds		= MeshBoundsCenter + MeshBoundsExtent;
	Result.DeformedMinBounds	= (float3)POSITIVE_INFINITY;
	Result.DeformedMaxBounds	= (float3)NEGATIVE_INFINITY;
	Result.CurSplineLength		= -1.0f; // Negative to mark as not started
	Result.MaxScaleXY			= 0.0f;

	Result.ShaderPrint.bIsActive	= false;
	Result.LocalToTWS				= (float4x4)0;

	return Result;
}

/** Solves post-deformed bounds of a slice of a spline mesh given the mesh-local bounds (iterative step) */
void SplineMeshDeformLocalBoundsStep(inout FSplineMeshDeformBoundsContext Context, half SplineDist)
{
	// Find the center, orientation, and scale of the slice at this point along the spline
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(Context.Params, SplineDist);
	const float AbsMaxScaleXY = max(abs(Slice.ScaleXY.x), abs(Slice.ScaleXY.y));
	const float3 XVec = Slice.Rot[1];
	const float3 YVec = Slice.Rot[2];

	float3 SliceMin, SliceMax;
#if SPLINE_MESH_DEFORM_BOUNDS_METHOD == SPLINE_MESH_DEFORM_BOUNDS_METHOD_QUADS
	// Calculate the mesh bounds along the X/Y of the slice
	const float2 MeshMinXY = Slice.ScaleXY * float2(dot(Context.Params.MeshX, Context.MeshMinBounds),
													 dot(Context.Params.MeshY, Context.MeshMinBounds));
	const float2 MeshMaxXY = Slice.ScaleXY * float2(dot(Context.Params.MeshX, Context.MeshMaxBounds),
													 dot(Context.Params.MeshY, Context.MeshMaxBounds));

	// Determine local-space AABB for a slice of the spline by transforming rect cross-section of bounds and take min/max
	const float3 RectPoints[4] =
	{
		Slice.Pos + XVec * MeshMinXY.x + YVec * MeshMinXY.y,
		Slice.Pos + XVec * MeshMinXY.x + YVec * MeshMaxXY.y,
		Slice.Pos + XVec * MeshMaxXY.x + YVec * MeshMaxXY.y,
		Slice.Pos + XVec * MeshMaxXY.x + YVec * MeshMinXY.y
	};

	SliceMin = min(min(RectPoints[0], RectPoints[1]), min(RectPoints[2], RectPoints[3]));
	SliceMax = max(max(RectPoints[0], RectPoints[1]), max(RectPoints[2], RectPoints[3]));
#elif SPLINE_MESH_DEFORM_BOUNDS_METHOD == SPLINE_MESH_DEFORM_BOUNDS_METHOD_SPHERES
	const float2 SphereOffsetXY = Slice.ScaleXY * float2(dot(Context.Params.MeshX, Context.MeshBoundsCenter),
														 dot(Context.Params.MeshY, Context.MeshBoundsCenter));
	const float3 SphereCenter = Slice.Pos + XVec * SphereOffsetXY.x + YVec * SphereOffsetXY.y;
	const float RadiusXY = AbsMaxScaleXY * abs(dot(Context.Params.MeshX + Context.Params.MeshY, Context.MeshBoundsExtent));

	SliceMin = SphereCenter - RadiusXY.xxx;
	SliceMax = SphereCenter + RadiusXY.xxx;
#else
	#error Invalid SPLINE_MESH_DEFORM_BOUNDS_METHOD
#endif

	// Extend current AABB and approximate spline length
	Context.DeformedMinBounds = min(Context.DeformedMinBounds, SliceMin);
	Context.DeformedMaxBounds = max(Context.DeformedMaxBounds, SliceMax);
	if (Context.CurSplineLength < 0.0f)
	{
		Context.CurSplineLength = 0.0f;
	}
	else
	{
		Context.CurSplineLength += length(Slice.Pos - Context.LastSlicePos);
	}
	Context.MaxScaleXY = max(Context.MaxScaleXY, AbsMaxScaleXY);
	Context.LastSlicePos = Slice.Pos;

	// Debug draw
	if (Context.ShaderPrint.bIsActive)
	{
#if SPLINE_MESH_DEFORM_BOUNDS_METHOD == SPLINE_MESH_DEFORM_BOUNDS_METHOD_QUADS
		float3 RectPointsTWS[4];
		UNROLL_N(4)
		for(int i = 0; i < 4; ++i)
		{
			RectPointsTWS[i] = mul(float4(RectPoints[i], 1.0f), Context.LocalToTWS).xyz;
		}

		const float4 MinColor = { 1.0f, 0.0f, 0.0f, 1.0f };
		const float4 MaxColor = { 0.0f, 1.0f, 0.0f, 1.0f };
		AddLineTWS(Context.ShaderPrint, RectPointsTWS[0], RectPointsTWS[1], MinColor);
		AddLineTWS(Context.ShaderPrint, RectPointsTWS[1], RectPointsTWS[2], MaxColor);
		AddLineTWS(Context.ShaderPrint, RectPointsTWS[2], RectPointsTWS[3], MaxColor);
		AddLineTWS(Context.ShaderPrint, RectPointsTWS[3], RectPointsTWS[0], MinColor);
#elif SPLINE_MESH_DEFORM_BOUNDS_METHOD == SPLINE_MESH_DEFORM_BOUNDS_METHOD_SPHERES
		const float3 SphereCenterTWS = mul(float4(SphereCenter, 1.0f), Context.LocalToTWS).xyz;
		const float4 SphereColor = { 0.0f, 1.0f, 0.0f, 1.0f };
		AddSphereTWS(Context.ShaderPrint, SphereCenterTWS, RadiusXY, SphereColor);
#endif
	}
}

/** Solves for approximate post-deformed bounds of a region of spline mesh given the mesh-local bounds */
FSplineMeshDeformedLocalBounds SplineMeshDeformLocalBounds(FSplineMeshDeformBoundsContext Context)
{
	// Find the min and max distance along the spline
	const half SplineDistMin = SplineMeshCalcSplineDistance(Context.Params, Context.MeshMinBounds);
	const half SplineDistMax = SplineMeshCalcSplineDistance(Context.Params, Context.MeshMaxBounds);

	const uint NUM_SLICE_SAMPLES = 8; // How many slices to sample along the length of the bounds
	half CurSplineDist = SplineDistMin;
	const half SplineDistStep = (SplineDistMax - SplineDistMin) / half(NUM_SLICE_SAMPLES - 1);

	// Sample at evenly-spaced intervals from min->max, accumulating the convex combination of slice bounds
	LOOP
	for (uint i = 0; i < NUM_SLICE_SAMPLES; ++i)
	{
		SplineMeshDeformLocalBoundsStep(Context, CurSplineDist);
		CurSplineDist += SplineDistStep;
	}

	FSplineMeshDeformedLocalBounds Result;
	Result.BoundsCenter = (Context.DeformedMinBounds + Context.DeformedMaxBounds) * 0.5f;
	Result.BoundsExtent = (Context.DeformedMaxBounds - Context.DeformedMinBounds) * 0.5f;

	// Allow cluster bounds extent to be scaled by a parameter as a kludge to fix any issues that might
	// occur with clusters dropping out due to inaccurate bounds calculated under extreme deformation.
	Result.BoundsExtent *= Context.Params.NaniteClusterBoundsScale;

	// Calculate an estimate for the largest scale in any one dimension caused by spline deformation and scaling.
	// This value is used to fudge the threshold at which to render Nanite clusters in HW to prevent issues.
	const float PreDeformLen = 2.0f * abs(dot(Context.Params.MeshDir, Context.MeshBoundsExtent));
	const float DeformLenScale = PreDeformLen == 0.0f ? 1.0f : Context.CurSplineLength * rcp(PreDeformLen);
	Result.MaxDeformScale = max(DeformLenScale, Context.MaxScaleXY);

	return Result;
}

FSplineMeshDeformedLocalBounds SplineMeshDeformLocalBounds(FSplineMeshShaderParams Params, float3 BoundsCenter, float3 BoundsExtent)
{
	return SplineMeshDeformLocalBounds(
		SplineMeshInitializeDeformBoundsContext(
			Params,
			BoundsCenter,
			BoundsExtent
		)
	);
}

FSplineMeshDeformedLocalBounds SplineMeshDeformLocalBoundsDebug(
	FSplineMeshShaderParams Params,
	FShaderPrintContext ShaderPrint,
	float4x4 LocalToTWS,
	float3 BoundsCenter,
	float3 BoundsExtent
)
{
	return SplineMeshDeformLocalBounds(
		SplineMeshInitializeDeformBoundsContext(
			Params,
			BoundsCenter,
			BoundsExtent,
			ShaderPrint,
			LocalToTWS
		)
	);
}

/**
 * Deforms a local-space mesh bounding sphere to *approximately* match its post-deformation equivalent along the spline.
 *
 * NOTE: This is currently only needed for Nanite LOD spheres, which are pretty tolerant to being very approximate
 * without having huge repercussions to the LOD quality. This is not a good solution if you need an accurate
 * transformation of local bounds (e.g. for use with culling). Also, this solution is not a monotonic transformation,
 * which is known to potentially create issues with Nanite's LOD selection.
 */
float4 SplineMeshDeformLODSphereBounds(FSplineMeshShaderParams Params, float4 LODSphere)
{
	// Find the center, orientation, and scale of the slice at the sphere's center point along the spline
	const half SplineDist = SplineMeshCalcSplineDistance(Params, LODSphere.xyz);
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(Params, SplineDist);
	const float2 SphereOffsetXY = Slice.ScaleXY * float2(dot(Params.MeshX, LODSphere.xyz),
														 dot(Params.MeshY, LODSphere.xyz));
	const float3 SpherePos = Slice.Pos + Slice.Rot[1] * SphereOffsetXY.x + Slice.Rot[2] * SphereOffsetXY.y;

	return float4(SpherePos, LODSphere.w * Params.MeshDeformScaleMinMax.y);
}