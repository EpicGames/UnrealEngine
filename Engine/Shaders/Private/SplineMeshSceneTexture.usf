// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SplineMeshCommon.ush"
#include "ComputeShaderUtils.ush"

RWTexture2D<float4> SplinePosTextureOut;
RWTexture2D<float4> SplineRotTextureOut;
StructuredBuffer<uint> InstanceIdLookup;
StructuredBuffer<uint> UpdateRequests;
uint NumUpdateRequests;
float TextureHeight;
float TextureHeightInv;

[numthreads(SPLINE_MESH_TEXEL_WIDTH, 1, 1)]
void FillTexture(uint3 GroupId : SV_GroupID, uint GroupThreadId : SV_GroupThreadID)
{
	GroupId = GetUnWrappedDispatchGroupId(GroupId);

	// 0 requests means we're filling the whole texture
	const uint SplineMeshIndex = NumUpdateRequests > 0 ? UpdateRequests[GroupId.x] : GroupId.x;
	const uint InstanceId = InstanceIdLookup[SplineMeshIndex];

	if (InstanceId == (uint)-1)
	{
		// Skip gaps in the lookup
		return;
	}

	FInstanceSceneData InstanceData = GetInstanceSceneDataUnchecked(InstanceId);
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

	if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_SPLINE_MESH) == 0 ||
		(InstanceData.Flags & INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION) == 0)
	{
		// Not a spline mesh instance?
		return;
	}

	// Determine the coord of the first texel of the spline and fill the row of texels with slice parameters
	const FSplineMeshShaderParams SplineMeshParams = SplineMeshLoadParamsFromInstancePayload(InstanceData);
	const float SplineDist = (GroupThreadId.x - SplineMeshParams.SplineDistToTexelOffset) / SplineMeshParams.SplineDistToTexelScale;
	const FSplineMeshSlice Slice = SplineMeshCalcSlice(SplineMeshParams, SplineDist);
	const uint2 PixelPos = SplineMeshParams.TextureCoord + uint2(GroupThreadId.x, 0);
	SplinePosTextureOut[PixelPos] = float4(Slice.Pos, 1.0f);
	SplineRotTextureOut[PixelPos] = Slice.Quat;
}