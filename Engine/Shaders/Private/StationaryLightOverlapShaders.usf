// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StationaryLightOverlapShaders.usf: Shaders for visualizing light overlap
=============================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "DynamicLightingCommon.ush" 

/** 1 if the light has a valid shadowmap channel. */
float bHasValidChannel;

#if RADIAL_ATTENUATION == 0
void OverlapPixelMain(
	float2 InUV : TEXCOORD0,
	float3 ScreenVector : TEXCOORD1,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = 0;
#if !FORWARD_SHADING
	FGBufferData GBuffer = GetGBufferData(InUV); 
	// Only light pixels marked as using deferred shading
	if( GBuffer.ShadingModelID > 0 )
#endif
	{
		// Output minor complexity to visualize overlap
		OutColor.rgb = .04f;

		if (bHasValidChannel < 1)
		{
			// Output a huge amount of complexity to make it clear that this light is an error
			OutColor.rgb = .4f;
		}
	}
}
#endif

#if RADIAL_ATTENUATION == 1
void OverlapPixelMain(
	float4 InScreenPosition : TEXCOORD0,
	in float4 SvPosition : SV_Position,		// after all interpolators
	out float4 OutColor : SV_Target0
	)
{
	OutColor = 0;
	float2 ScreenUV = SvPositionToBufferUV(SvPosition);
	SvPosition.z = LookupDeviceZ(ScreenUV);
	
#if !FORWARD_SHADING
	FGBufferData GBuffer = GetGBufferData(ScreenUV); 
	// Only light pixels marked as using deferred shading
	BRANCH if( GBuffer.ShadingModelID > 0 )
#endif
{
		float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);

		float3 ToLight = DeferredLightUniforms.TranslatedWorldPosition - TranslatedWorldPosition;

		float SpotFalloff = SpotAttenuation(normalize(ToLight), -DeferredLightUniforms.Direction, DeferredLightUniforms.SpotAngles);

		float RadialAttenuationRatio = length(TranslatedWorldPosition - DeferredLightUniforms.TranslatedWorldPosition) * DeferredLightUniforms.InvRadius;

		if (RadialAttenuationRatio < 1 && SpotFalloff > .0001f)
		{
			// Output minor complexity to visualize overlap
			OutColor.rgb = .04f;

			if (bHasValidChannel < 1)
			{
				// Output a huge amount of complexity to make it clear that this light is an error
				OutColor.rgb = .4f;
			}
		}
	}
}
#endif 