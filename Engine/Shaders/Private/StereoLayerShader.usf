// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	StereoLayerShader.usf: Stereo layer shader source.
=============================================================================*/

#include "Common.ush"

float2 InQuadAdjust;
float4 InUVAdjust;
float4x4 InViewProjection;
float4x4 InWorld;

void MainVS(
	    float2 InPosition : ATTRIBUTE0,
	    float2 InUV       : ATTRIBUTE1,
	out float2 UV         : TEXCOORD0,
	out float4 Position   : SV_POSITION
	)
{
	float4 LocalPosition = float4((-1 + 2 * InPosition) * InQuadAdjust, 0, 1);
	float4 WorldPosition = mul(LocalPosition, InWorld);
	Position = mul(WorldPosition, InViewProjection);
#if COMPILER_GLSL_ES3_1 && !MOBILE_EMULATION
	// The GLES target has a flipped Y axis
	Position.y = -Position.y;
#endif
	UV = (InUVAdjust.xy + (InUV * InUVAdjust.zw));
	UV.y = 1 - UV.y;
}

Texture2D InTexture;
SamplerState InTextureSampler;
float InIsOpaque;

float CalculateAlpha(float4 OutColor)
{
	// Don't export alpha for opaque layers
	return (OutColor.a * (1.0 - InIsOpaque) + InIsOpaque);
}

void MainPS_Texture2D(
	    float2 UV       : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = Texture2DSample(InTexture, InTextureSampler, UV);
	OutColor.a = CalculateAlpha(OutColor);
}

TextureExternal InExternalTexture;

void MainPS_TextureExternal(
	    float2 UV       : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = TextureExternalSample(InExternalTexture, InTextureSampler, UV);
	OutColor.a = CalculateAlpha(OutColor);
}
