// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "../Common.ush"

#define LUMEN_INVALID_PACKED_PIXEL_DATA 0x30
#define MEGALIGHTS_INVALID_PACKED_PIXEL_DATA 0x30

// Used for storing normal and shading info history data and used by:
// * LumenScreenProbeGather
// * MegaLightDenoiserTemporal
struct FNormalAndShadingInfo
{
	float3 Normal;
	bool bIsHair;
	bool bHasBackfaceDiffuse;
};

float4 PackNormalAndShadingInfo(FNormalAndShadingInfo In)
{
	// Storage format 10:10:10:2
	const float3 PackedN =  In.Normal * 0.5 + 0.5;

	uint PackedA = In.bIsHair ? 0x1 : 0;
	PackedA |= In.bHasBackfaceDiffuse ? 0x2 : 0;

	return float4(PackedN, (PackedA + 0.5f) / 4.0f);
}

FNormalAndShadingInfo UnpackNormalAndShadingInfo(float4 In)
{
	uint PackedA = In.w * 3.0f + 0.5f;

	// Storage format 10:10:10:2
	FNormalAndShadingInfo Out;
	Out.Normal = In.xyz * 2 - 1;
	Out.bIsHair = PackedA & 0x1 ? true : false;
	Out.bHasBackfaceDiffuse = PackedA & 0x2 ? true : false;
	return Out;
}

uint2 EncodeHistoryScreenCoord(float2 HistoryScreenCoord, float2 SubPixelGridSize)
{
	uint2 EncodeCoord = clamp(HistoryScreenCoord * SubPixelGridSize + 0.5f, 0.0f, 65535.0f);
	return EncodeCoord;
}

float4 DecodeHistoryScreenCoord(uint2 EncodeCoord, uint2 HistoryScreenCoordShift, float2 InvSubPixelGridSize)
{
	uint2 HistoryScreenCoord = EncodeCoord >> HistoryScreenCoordShift;
	uint2 SubPixelCoord = EncodeCoord - (HistoryScreenCoord << HistoryScreenCoordShift);
	float2 BilinearWeights = float2(SubPixelCoord) * InvSubPixelGridSize;
	return float4(HistoryScreenCoord, BilinearWeights);
}

struct FLumenPackedPixelData
{
	uint Packed;

	void SetHistorySampleValidity(bool4 bSampleValid)
	{
		for (uint HistoryIndex = 0; HistoryIndex < 4; ++HistoryIndex)
		{
			Packed |= bSampleValid[HistoryIndex] ? 1u << HistoryIndex : 0;
		}
	}

	bool4 GetHistorySampleValidity()
	{
		bool4 bHistoryValid;
		for (uint HistoryIndex = 0; HistoryIndex < 4; ++HistoryIndex)
		{
			bHistoryValid[HistoryIndex] = (Packed & (1u << HistoryIndex)) != 0;
		}
		return bHistoryValid;
	}

	bool AnyHistoryValid()
	{
		return (Packed & 0xF) != 0;
	}

	void SetStochasticSampleOffset(uint2 Offset, bool bCanReconstruct)
	{
		if (bCanReconstruct)
		{
			Packed |= Offset.x << 4;
			Packed |= Offset.y << 5;
			Packed |= 1u << 6;
		}
	}

	uint2 GetStochasticSampleOffset()
	{
		return select(bool2(Packed & (1u << 4), Packed & (1u << 5)), 1u, 0u);
	}

	bool HasValidStochasticSample()
	{
		return (Packed & (1u << 6)) != 0;
	}

	void SetHasBackfaceDiffuse(bool bBackfaceDiffuse)
	{
		Packed |= bBackfaceDiffuse ? 1u << 7 : 0;
	}

	bool HasBackfaceDiffuse()
	{
		return (Packed & (1u << 7)) != 0;
	}

	bool IsValid()
	{
		return Packed != LUMEN_INVALID_PACKED_PIXEL_DATA;
	}
};

struct FMegaLightsPackedPixelData
{
	uint Packed;

	void SetHistorySampleValidity(bool4 bSampleValid)
	{
		for (uint HistoryIndex = 0; HistoryIndex < 4; ++HistoryIndex)
		{
			Packed |= bSampleValid[HistoryIndex] ? 1u << HistoryIndex : 0;
		}
	}

	bool4 GetHistorySampleValidity()
	{
		bool4 bHistoryValid;
		for (uint HistoryIndex = 0; HistoryIndex < 4; ++HistoryIndex)
		{
			bHistoryValid[HistoryIndex] = (Packed & (1u << HistoryIndex)) != 0;
		}
		return bHistoryValid;
	}

	bool AnyHistoryValid()
	{
		return (Packed & 0xF) != 0;
	}

	void SetStochasticSampleOffset(uint2 Offset, bool bCanReconstruct)
	{
		if (bCanReconstruct)
		{
			Packed |= Offset.x << 4;
			Packed |= Offset.y << 5;
			Packed |= 1u << 6;
		}
	}

	uint2 GetStochasticSampleOffset()
	{
		return select(bool2(Packed & (1u << 4), Packed & (1u << 5)), 1u, 0u);
	}

	int2 GetStochasticSampleOffset(int2 SampleOffsets[4])
	{
		uint SampleOffsetIndex = BitFieldExtractU32(Packed, 2, 4);
		return SampleOffsets[SampleOffsetIndex];
	}

	bool HasValidStochasticSample()
	{
		return (Packed & (1u << 6)) != 0;
	}

	void SetBicubicFootprintValid(bool bAnyValid)
	{
		Packed |= bAnyValid ? 1u << 7 : 0;
	}

	bool IsBicubicFootprintValid()
	{
		return (Packed & (1u << 7)) != 0;
	}

	bool IsValid()
	{
		return Packed != MEGALIGHTS_INVALID_PACKED_PIXEL_DATA;
	}
};

// Float R11G11B10 max
#define INVALID_LIGHTING half3(65024.0, 65024.0, 64512.0)

bool IsLightingValid(half3 V)
{
	return !all(V == INVALID_LIGHTING);
}

half3 LumenRGBToYCoCg(half3 RGB)
{
	half Y  = dot(RGB, half3( 0.25, 0.5, 0.25));
	half Co = dot(RGB, half3( 0.5, 0, -0.5));
	half Cg = dot(RGB, half3(-0.25, 0.5, -0.25));
	
	half3 YCoCg = half3(Y, Co, Cg);
	return YCoCg;
}

half3 LumenYCoCgToRGB(half3 YCoCg)
{
	half Y  = YCoCg.x;
	half Co = YCoCg.y;
	half Cg = YCoCg.z;

	half R = Y + Co - Cg;
	half G = Y + Cg;
	half B = Y - Co - Cg;

	half3 RGB = half3(R, G, B);
	return RGB;
}

// [Moving Frostbite to Physically Based Rendering 3.0]
// Approximates factor in Lerp(N, R, factor) = dominant direction of a GGX lobe
float GetSpecularDominantDirFactor(float Roughness)
{
	float S = saturate(1.0f - Roughness);
	return S * (sqrt(S) + Roughness);
}

// [Moving Frostbite to Physically Based Rendering 3.0]
float GetSpecularLobeHalfAngle(float Roughness)
{
	float FractionOfPreservedEnergy = 0.75;
	float LobeHalfAngle = AtanFast((Pow2(Roughness) * FractionOfPreservedEnergy) / (1.0 - FractionOfPreservedEnergy));
	return LobeHalfAngle;
}

float DegreesToRadians(float InDeg)
{
	return InDeg / 180.0 * PI;
}
