// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

float4 FirstPersonWorldSpaceRepresentationBounds; // Collective bounding sphere for all the invisible primitives tagged as FirstPersonWorldSpaceRepresentation
float FirstPersonMinimumHitDistanceOnScreenTraceMiss;

// First person world space representation primitives are invisible to the camera but visible in shadows and HWRT reflections.
// Screen traces can miss intersections with these invisible meshes, so when HWRT picks up, we end up seeing holes and other artifacts
// in the reflected meshes. To work around this issue, we move the ray position at which we hand off to HWRT outside the bounds of the
// invisible geometry such that HWRT has a chance to intersect it.
float ClipFirstPersonScreenSpaceRay(float3 TranslatedWorldPosition, float3 RayDirection, float RayT, bool bHit)
{
#if HAS_FIRST_PERSON_GBUFFER_BIT
	if (!bHit && FirstPersonWorldSpaceRepresentationBounds.w > 0.0)
	{
		const float2 Intersections = RayIntersectSphere(TranslatedWorldPosition, RayDirection, FirstPersonWorldSpaceRepresentationBounds);
		if (all(Intersections != -1.0f))
		{
			RayT = min(RayT, max(Intersections.x, FirstPersonMinimumHitDistanceOnScreenTraceMiss));
		}
	}
#endif

	return RayT;
}