// Copyright Epic Games, Inc. All Rights Reserved.

// For now only Substrate Format=0 uses this path, which does not support Special complex classification
#define SUBSTRATE_COMPLEXSPECIALPATH 0

#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE 1
#define USE_HAIR_COMPLEX_TRANSMITTANCE 1

#include "../Common.ush"
#include "../LightGridCommon.ush"
#include "../Lumen/LumenMaterial.ush"
#include "../Lumen/LumenReflectionCommon.ush"
#include "../Lumen/LumenScreenProbeTileClassication.ush"
#include "../MegaLights/MegaLightsTileClassification.ush"
#include "StochasticLightingUpsample.ush"
#include "StochasticLightingCommon.ush"

MAX_OCCUPANCY

#define DOWNSAMPLE_FACTOR	2
#define TILE_SIZE			8
#define TILE_SIZE_DIV_SHIFT	3

#define STOCHASTIC_SAMPLE_OFFSET_2x1 1
#define STOCHASTIC_SAMPLE_OFFSET_2x2 2

#define USE_CLOSURECOUNT_HISTORY_REJECTION (SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT == 1 && MATERIAL_SOURCE == MATERIAL_SOURCE_GBUFFER)

Texture2D DepthHistoryTexture;
Texture2D<float4> NormalAndShadingInfoHistory;
Texture2D<uint> ClosureHistory;
Texture2D<uint> MegaLightsNumFramesAccumulatedHistory;

#if TILE_INDIRECT_DISPATCH
StructuredBuffer<uint> TileIndirectDispatchData;
#endif

RWTexture2D<float> RWDepthTexture;
RWTexture2D<float4> RWNormalTexture;
RWTexture2D<uint> RWClosureTexture;
RWTexture2DArray<uint> RWLumenTileBitmask;
RWTexture2D<uint> RWMegaLightsTileBitmask;
RWTexture2D<uint2> RWEncodedHistoryScreenCoord;
RWTexture2DArray<uint> RWLumenPackedPixelData;
RWTexture2D<uint> RWMegaLightsPackedPixelData;

uint EnableTexturedRectLights;
float4 HistoryScreenPositionScaleBias;
float4 HistoryUVMinMax;
float4 HistoryGatherUVMinMax;
float4 HistoryBufferSizeAndInvSize;
float4 HistorySubPixelGridSizeAndInvSize;
float LumenHistoryDistanceThreshold;
float LumenHistoryDistanceThresholdForFoliage;
float LumenHistoryNormalCosThreshold;
float MegaLightsHistoryDistanceThreshold;
uint2 HistoryScreenCoordDecodeShift;
uint2 DownsampledViewMin2x1;
uint2 DownsampledViewMin2x2;
uint2 DownsampledViewSize2x1;
uint2 DownsampledViewSize2x2;
uint LumenStochasticSampleMode;
uint MegaLightsStochasticSampleMode;
uint OutputDownsampledDepthAndNormal;

#if TILE_CLASSIFY_LUMEN || TILE_CLASSIFY_MEGALIGHTS || TILE_CLASSIFY_SUBSTRATE
#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	groupshared uint SharedTileBitmask;
#else
	groupshared uint SharedTileBitmask[THREADGROUP_SIZE * THREADGROUP_SIZE];
#endif

	#define TILE_BITMASK_INDEX_LUMEN 0
	#define TILE_BITMASK_INDEX_MEGALIGHTS 1
	#define TILE_BITMASK_INDEX_SUBSTRATE 2
	#define TILE_BITMASK_BIT_COUNT 10
	#define TILE_BITMASK_BIT_MASK  0x3FF
	#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	#define WriteSharedTileBitmask(LinearThreadIndex, InBitmask, InBitmaskIndex) 				{ GroupTileBitmask |= (TILE_BITMASK_BIT_MASK & InBitmask) << (InBitmaskIndex*TILE_BITMASK_BIT_COUNT); }
	#define ReadSharedTileBitmask(LinearThreadIndex, InBitmaskIndex) 			 				BitFieldExtractU32(GroupTileBitmask, TILE_BITMASK_BIT_COUNT, InBitmaskIndex * TILE_BITMASK_BIT_COUNT)
	#else
	void WriteSharedTileBitmask(uint LinearThreadIndex, uint InBitmask, uint InBitmaskIndex) 	{ SharedTileBitmask[LinearThreadIndex] |= (TILE_BITMASK_BIT_MASK & InBitmask) << (InBitmaskIndex*TILE_BITMASK_BIT_COUNT); }
	uint ReadSharedTileBitmask(uint LinearThreadIndex, uint InBitmaskIndex) 					{ return BitFieldExtractU32(SharedTileBitmask[LinearThreadIndex], TILE_BITMASK_BIT_COUNT, InBitmaskIndex*TILE_BITMASK_BIT_COUNT); }
	#endif
#endif

RWTexture2D<float> RWDownsampledSceneDepth2x1;
RWTexture2D<float> RWDownsampledSceneDepth2x2;
RWTexture2D<UNORM float3> RWDownsampledWorldNormal2x1;
RWTexture2D<UNORM float3> RWDownsampledWorldNormal2x2;

groupshared float SharedSceneDepth2x1[6][10];
groupshared float3 SharedEncodedNormal2x1[6][10];
groupshared float SharedSceneDepth2x2[6][6];
groupshared float3 SharedEncodedNormal2x2[6][6];

void StoreDepthToCache(uint2 SharedCoord, const uint2 DownsampleFactor, float Depth)
{
	if (all(DownsampleFactor == 2))
	{
		SharedSceneDepth2x2[SharedCoord.x][SharedCoord.y] = Depth;
	}
	else
	{
		SharedSceneDepth2x1[SharedCoord.x][SharedCoord.y] = Depth;
	}
}

float LoadDepthFromCache(uint2 SharedCoord, const uint2 DownsampleFactor)
{
	if (all(DownsampleFactor == 2))
	{
		return SharedSceneDepth2x2[SharedCoord.x][SharedCoord.y];
	}
	else
	{
		return SharedSceneDepth2x1[SharedCoord.x][SharedCoord.y];
	}
}

void StoreNormalToCache(uint2 SharedCoord, const uint2 DownsampleFactor, float3 Normal)
{
	if (all(DownsampleFactor == 2))
	{
		SharedEncodedNormal2x2[SharedCoord.x][SharedCoord.y] = Normal;
	}
	else
	{
		SharedEncodedNormal2x1[SharedCoord.x][SharedCoord.y] = Normal;
	}
}

float3 LoadNormalFromCache(uint2 SharedCoord, const uint2 DownsampleFactor)
{
	if (all(DownsampleFactor == 2))
	{
		return SharedEncodedNormal2x2[SharedCoord.x][SharedCoord.y];
	}
	else
	{
		return SharedEncodedNormal2x1[SharedCoord.x][SharedCoord.y];
	}
}

float GetNormalWeight(float3 SceneWorldNormal, float3 SampleWorldNormal)
{
	float AngleBetweenNormals = acosFast(saturate(dot(SampleWorldNormal, SceneWorldNormal)));
	float NormalWeight = 1.0f - saturate(AngleBetweenNormals);

	return Pow2(NormalWeight);
}

uint2 ThreadIndexToBorderCoord(uint LinearThreadIndex, const uint2 DownsampleFactor)
{
	const uint TileBorder = 1;
	const uint2 TileSize = TILE_SIZE / DownsampleFactor + TileBorder * 2;
	const uint NumPartialRows = TileSize.y - TileBorder * 2;
	const uint PixelsPerPartialRow = TileBorder * 2;
	
	uint2 BorderCoord;
	if (LinearThreadIndex < TileSize.x * TileBorder)
	{
		BorderCoord = uint2(LinearThreadIndex % TileSize.x, LinearThreadIndex / TileSize.x);
	}
	else if (LinearThreadIndex - TileSize.x * TileBorder < NumPartialRows * PixelsPerPartialRow)
	{
		uint LocalIndex = LinearThreadIndex - TileSize.x * TileBorder;
		BorderCoord = uint2(LocalIndex % PixelsPerPartialRow, LocalIndex / PixelsPerPartialRow);
		BorderCoord.x += BorderCoord.x < TileBorder ? 0 : (TileSize.x - TileBorder * 2);
		BorderCoord.y += TileBorder;
	}
	else
	{
		uint LocalIndex = LinearThreadIndex - TileSize.x * TileBorder - NumPartialRows * PixelsPerPartialRow;
		BorderCoord = uint2(LocalIndex % TileSize.x, TileSize.y - TileBorder + LocalIndex / TileSize.x);
	}

	return BorderCoord;
}

uint2 GetDownsampleJitter(uint2 DownsampledCoord, const uint2 DownsampleFactor)
{
	uint2 Jitter = 0;

	if (all(DownsampleFactor == 2))
	{
		Jitter = GetDownsampleJitter2x2(DownsampledCoord);
	}
	else if (all(DownsampleFactor == uint2(2, 1)))
	{
		Jitter = GetDownsampleJitter2x1(DownsampledCoord);
	}
	
	return Jitter;
}

void DownsampleDepthAndNormal(
	uint2 GroupId,
	uint2 GroupThreadId,
	uint2 ScreenCoord,
	FLumenMaterialData Material,
	const uint2 DownsampleFactor,
	RWTexture2D<float> RWDownsampledDepth,
	RWTexture2D<UNORM float3> RWDownsampledNormal)
{
	const uint TileBorder = 1;
	const uint2 TileSize = TILE_SIZE / DownsampleFactor + TileBorder * 2;

	uint2 DownsampledScreenCoord = ScreenCoord / DownsampleFactor;
	uint2 Jitter = GetDownsampleJitter(DownsampledScreenCoord, DownsampleFactor);

	if (all(Jitter == ScreenCoord % DownsampleFactor))
	{
		if (OutputDownsampledDepthAndNormal > 0)
		{
			RWDownsampledDepth[DownsampledScreenCoord] = Material.IsValid() ? Material.SceneDepth : -1.0f;
			RWDownsampledNormal[DownsampledScreenCoord] = EncodeNormal(Material.WorldNormalForPositionBias);
		}

		uint2 SharedCoord = GroupThreadId / DownsampleFactor + TileBorder;
		StoreDepthToCache(SharedCoord, DownsampleFactor, Material.IsValid() ? Material.SceneDepth : -1.0f);
		StoreNormalToCache(SharedCoord, DownsampleFactor, Material.WorldNormalForPositionBias);
	}

	const uint NumBorderPixels = (TileSize.x + TileSize.y - TileBorder * 2) * TileBorder * 2;
	uint LinearThreadIndex = GroupThreadId.x + THREADGROUP_SIZE * GroupThreadId.y;

	if (LinearThreadIndex < NumBorderPixels)
	{
		uint2 BorderSharedCoord = ThreadIndexToBorderCoord(LinearThreadIndex, DownsampleFactor);
		uint2 BorderDownsampledCoord = (View.ViewRectMinAndSize.xy + GroupId * TILE_SIZE) / DownsampleFactor - TileBorder + BorderSharedCoord;
		uint2 BorderScreenCoord = BorderDownsampledCoord * DownsampleFactor + GetDownsampleJitter(BorderDownsampledCoord, DownsampleFactor);

		bool bCanLoadFromCache = false;
	#if STOCHASTIC_SAMPLE_OFFSET == (STOCHASTIC_SAMPLE_OFFSET_2x1 | STOCHASTIC_SAMPLE_OFFSET_2x2)
		if (all(DownsampleFactor == uint2(2, 1)))
		{
			uint2 DownsampledCoord2x2 = BorderDownsampledCoord / uint2(1, 2);
			uint2 ScreenCoord2x2 = DownsampledCoord2x2 * 2 + GetDownsampleJitter2x2(DownsampledCoord2x2);
			bCanLoadFromCache = all(BorderScreenCoord == ScreenCoord2x2);
		}
	#endif

		float BorderDepth = -1.0f;
		float3 BorderNormal = 0.0f;
		if (bCanLoadFromCache)
		{
			uint2 SharedCoord2x2 = BorderSharedCoord / uint2(1, 2);
			BorderDepth = LoadDepthFromCache(SharedCoord2x2, uint2(2, 2));
			BorderNormal = LoadNormalFromCache(SharedCoord2x2, uint2(2, 2));
		}
		else if (all(BorderScreenCoord - View.ViewRectMinAndSize.xy < View.ViewRectMinAndSize.zw))
		{
			FLumenMaterialCoord BorderCoord = (FLumenMaterialCoord)0;
			BorderCoord.SvPosition = BorderScreenCoord;
			FLumenMaterialData BorderMaterial = ReadMaterialData(BorderCoord, MaxRoughnessToTrace);

			BorderDepth = BorderMaterial.IsValid() ? BorderMaterial.SceneDepth : -1.0f;
			BorderNormal = BorderMaterial.WorldNormalForPositionBias;
		}

		StoreDepthToCache(BorderSharedCoord, DownsampleFactor, BorderDepth);
		StoreNormalToCache(BorderSharedCoord, DownsampleFactor, BorderNormal);
	}
}

bool IsValidDownsampledCoord(uint2 DownsampledScreenCoord, const uint2 DownsampleFactor)
{
	if (all(DownsampleFactor == 2))
	{
		return all(DownsampledScreenCoord.xy < DownsampledViewMin2x2 + DownsampledViewSize2x2);
	}
	else
	{
		return all(DownsampledScreenCoord.xy < DownsampledViewMin2x1 + DownsampledViewSize2x1);
	}
}

float4 ComputeUpsampleWeights(uint2 GroupThreadId, uint2 ScreenCoord, float2 ScreenUV, float2 ScreenPosition, float3 TranslatedWorldPosition, FLumenMaterialData Material, const uint2 DownsampleFactor)
{
	const uint TileBorder = 1;
	const bool bDownsample2x1 = all(DownsampleFactor == uint2(2, 1));

	int2 SampleOffsets[4];
	int2 SharedCoord00;
	int2 DownsampledCoord00;

	if (bDownsample2x1)
	{
		SampleOffsets[0] = int2(0, 0);
		SampleOffsets[1] = int2(ScreenCoord.x % DownsampleFactor.x == 0 ? -1 : 1, 0);
		SampleOffsets[2] = int2(0, -1);
		SampleOffsets[3] = int2(0, 1);

		SharedCoord00 = GroupThreadId / DownsampleFactor + TileBorder;
		DownsampledCoord00 = ScreenCoord / DownsampleFactor;
	}
	else
	{
		SampleOffsets[0] = int2(0, 0);
		SampleOffsets[1] = int2(1, 0);
		SampleOffsets[2] = int2(0, 1);
		SampleOffsets[3] = int2(1, 1);

		SharedCoord00 = (GroupThreadId + TileBorder * 2 - 1) / DownsampleFactor;
		DownsampledCoord00 = floor(ScreenUV * View.BufferSizeAndInvSize.xy / DownsampleFactor - 0.5f);
	}

	float4 CornerDepths;
	CornerDepths.x = LoadDepthFromCache(SharedCoord00 + SampleOffsets[0], DownsampleFactor);
	CornerDepths.y = LoadDepthFromCache(SharedCoord00 + SampleOffsets[1], DownsampleFactor);
	CornerDepths.z = LoadDepthFromCache(SharedCoord00 + SampleOffsets[2], DownsampleFactor);
	CornerDepths.w = LoadDepthFromCache(SharedCoord00 + SampleOffsets[3], DownsampleFactor);

	int2 ScreenCoordOffset = ScreenCoord - DownsampledCoord00 * DownsampleFactor;
	int2 SampleScreenOffset00 = GetDownsampleJitter(DownsampledCoord00 + SampleOffsets[0], DownsampleFactor) + SampleOffsets[0] * 2 - ScreenCoordOffset;
	int2 SampleScreenOffset10 = GetDownsampleJitter(DownsampledCoord00 + SampleOffsets[1], DownsampleFactor) + SampleOffsets[1] * 2 - ScreenCoordOffset;
	int2 SampleScreenOffset01 = GetDownsampleJitter(DownsampledCoord00 + SampleOffsets[2], DownsampleFactor) + SampleOffsets[2] * 2 - ScreenCoordOffset;
	int2 SampleScreenOffset11 = GetDownsampleJitter(DownsampledCoord00 + SampleOffsets[3], DownsampleFactor) + SampleOffsets[3] * 2 - ScreenCoordOffset;

	// Triangle filter weights between pixel and 4 samples
	float4 UpsampleWeights;
	if (bDownsample2x1)
	{
		UpsampleWeights = select(SampleScreenOffset00.x == 0, float4(1.0, 0.0, 0.0, 0.0), float4(0.25, 0.25, 0.25, 0.25));
	}
	else
	{
		UpsampleWeights.x = (2.0f - abs(SampleScreenOffset00.x)) * (2.0f - abs(SampleScreenOffset00.y));
		UpsampleWeights.y = (2.0f - abs(SampleScreenOffset10.x)) * (2.0f - abs(SampleScreenOffset10.y));
		UpsampleWeights.z = (2.0f - abs(SampleScreenOffset01.x)) * (2.0f - abs(SampleScreenOffset01.y));
		UpsampleWeights.w = (2.0f - abs(SampleScreenOffset11.x)) * (2.0f - abs(SampleScreenOffset11.y));
	}

	float4 DepthWeights = 1.0f;	
	{
		float4 ScenePlane = float4(Material.WorldNormalForPositionBias, dot(TranslatedWorldPosition, Material.WorldNormalForPositionBias));

		float2 ScreenPosition00 = ScreenPosition + SampleScreenOffset00 * View.BufferSizeAndInvSize.zw / View.ScreenPositionScaleBias.xy;
		float2 ScreenPosition10 = ScreenPosition + SampleScreenOffset10 * View.BufferSizeAndInvSize.zw / View.ScreenPositionScaleBias.xy;
		float2 ScreenPosition01 = ScreenPosition + SampleScreenOffset01 * View.BufferSizeAndInvSize.zw / View.ScreenPositionScaleBias.xy;
		float2 ScreenPosition11 = ScreenPosition + SampleScreenOffset11 * View.BufferSizeAndInvSize.zw / View.ScreenPositionScaleBias.xy;

		float3 Position00 = mul(float4(GetScreenPositionForProjectionType(ScreenPosition00, CornerDepths.x), CornerDepths.x, 1), View.ScreenToTranslatedWorld).xyz;
		float3 Position10 = mul(float4(GetScreenPositionForProjectionType(ScreenPosition10, CornerDepths.y), CornerDepths.y, 1), View.ScreenToTranslatedWorld).xyz;
		float3 Position01 = mul(float4(GetScreenPositionForProjectionType(ScreenPosition01, CornerDepths.z), CornerDepths.z, 1), View.ScreenToTranslatedWorld).xyz;
		float3 Position11 = mul(float4(GetScreenPositionForProjectionType(ScreenPosition11, CornerDepths.w), CornerDepths.w, 1), View.ScreenToTranslatedWorld).xyz;

		float4 PlaneDistances;
		PlaneDistances.x = abs(dot(float4(Position00, -1), ScenePlane));
		PlaneDistances.y = abs(dot(float4(Position10, -1), ScenePlane));
		PlaneDistances.z = abs(dot(float4(Position01, -1), ScenePlane));
		PlaneDistances.w = abs(dot(float4(Position11, -1), ScenePlane));

		float4 RelativeDepthDifference = PlaneDistances / Material.SceneDepth;

		DepthWeights = select(CornerDepths > 0.0f, exp2(-5000.0f * (RelativeDepthDifference * RelativeDepthDifference)), 0.0f);
	}
	UpsampleWeights *= DepthWeights;

	float4 NormalWeights = 1.0f;
	{
		float3 SampleWorldNormal00 = LoadNormalFromCache(SharedCoord00 + SampleOffsets[0], DownsampleFactor);
		float3 SampleWorldNormal10 = LoadNormalFromCache(SharedCoord00 + SampleOffsets[1], DownsampleFactor);
		float3 SampleWorldNormal01 = LoadNormalFromCache(SharedCoord00 + SampleOffsets[2], DownsampleFactor);
		float3 SampleWorldNormal11 = LoadNormalFromCache(SharedCoord00 + SampleOffsets[3], DownsampleFactor);

		NormalWeights.x = GetNormalWeight(Material.WorldNormalForPositionBias, SampleWorldNormal00);
		NormalWeights.y = GetNormalWeight(Material.WorldNormalForPositionBias, SampleWorldNormal10);
		NormalWeights.z = GetNormalWeight(Material.WorldNormalForPositionBias, SampleWorldNormal01);
		NormalWeights.w = GetNormalWeight(Material.WorldNormalForPositionBias, SampleWorldNormal11);
	}
	UpsampleWeights *= NormalWeights;

	// Skip out of view samples
	UpsampleWeights.x = IsValidDownsampledCoord(DownsampledCoord00 + SampleOffsets[0], DownsampleFactor) ? UpsampleWeights.x : 0.0f;
	UpsampleWeights.y = IsValidDownsampledCoord(DownsampledCoord00 + SampleOffsets[1], DownsampleFactor) ? UpsampleWeights.y : 0.0f;
	UpsampleWeights.z = IsValidDownsampledCoord(DownsampledCoord00 + SampleOffsets[2], DownsampleFactor) ? UpsampleWeights.z : 0.0f;
	UpsampleWeights.w = IsValidDownsampledCoord(DownsampledCoord00 + SampleOffsets[3], DownsampleFactor) ? UpsampleWeights.w : 0.0f;

	return UpsampleWeights;
}

#if TILE_CLASSIFY_SUBSTRATE
	RWBuffer<uint> TileDrawIndirectDataBufferUAV;
	RWBuffer<uint> TileListBufferUAV;
	uint bRectPrimitive;
	uint TileEncoding;
	uint4 TileListBufferOffsets[SUBSTRATE_TILE_TYPE_COUNT];
	uint GetTileListBufferOffsets(uint Type)
	{
		return TileListBufferOffsets[Type].x;
	}
#endif

float4 ConvertFromDeviceZ4(float4 DeviceZ4)
{
	float4 V;
	V.x = ConvertFromDeviceZ(DeviceZ4.x);
	V.y = ConvertFromDeviceZ(DeviceZ4.y);
	V.z = ConvertFromDeviceZ(DeviceZ4.z);
	V.w = ConvertFromDeviceZ(DeviceZ4.w);
	return V;
}

// 2x2 bilinear taps stored in one bool4 like this:
// x y
// z w
struct FHistoryTaps4
{
	bool4 Quad;

	void Init()
	{
		Quad = true;
	}

	void SetBilinear(bool4 bValid)
	{
		Quad = bValid;
	}

	void TestBilinear(bool4 bValid)
	{
		Quad = and(Quad, bValid);
	}

	bool4 GetBilinear()
	{
		return Quad;
	}
};

// 12 taps stored in 4 x bool3
// 12 used for the Catmull-Rom approximation
// Center 4 used for the 2x2 bilinear filter
//    0x | 1x
// 0y 0z | 1y 1z
// -------------
// 2x 2y | 3x 3y
//    2z | 3z
struct FHistoryTaps12
{
	bool3 Quad0;
	bool3 Quad1;
	bool3 Quad2;
	bool3 Quad3;

	void Init()
	{
		Quad0 = true;
		Quad1 = true;
		Quad2 = true;
		Quad3 = true;
	}

	void SetBicubic(bool3 bValid0, bool3 bValid1, bool3 bValid2, bool3 bValid3)
	{
		Quad0 = bValid0;
		Quad1 = bValid1;
		Quad2 = bValid2;
		Quad3 = bValid3;
	}

	void TestBicubic(bool3 bValid0, bool3 bValid1, bool3 bValid2, bool3 bValid3)
	{
		Quad0 = and(Quad0, bValid0);
		Quad1 = and(Quad1, bValid1);
		Quad2 = and(Quad2, bValid2);
		Quad3 = and(Quad3, bValid3);
	}

	void SetBilinear(bool4 bValid)
	{
		Quad0.z = bValid.x;
		Quad1.y = bValid.y;
		Quad2.y = bValid.z;
		Quad3.x = bValid.w;
	}
	
	void TestBilinear(bool4 bValid)
	{
		Quad0.z = and(Quad0.z, bValid.x);
		Quad1.y = and(Quad1.y, bValid.y);
		Quad2.y = and(Quad2.y, bValid.z);
		Quad3.x = and(Quad3.x, bValid.w);
	}

	bool4 GetBilinear()
	{
		return bool4(Quad0.z, Quad1.y, Quad2.y, Quad3.x);
	}

	bool IsBicubicValid()
	{
		return all(and(and(and(Quad0, Quad1), Quad2), Quad3));
	}
};

/**
 * Load GBuffer data once and transform it for subsequent lighting passes
 * This includes full res depth and normal copy for opaque before it gets overwritten by water or other translucency writing depth
 */
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void StochasticLightingTileClassificationMarkCS(
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint2 DispatchThreadId : SV_DispatchThreadID)
{
	uint LinearThreadIndex = GroupThreadId.x + THREADGROUP_SIZE * GroupThreadId.y;

#if TILE_CLASSIFY_LUMEN || TILE_CLASSIFY_MEGALIGHTS || TILE_CLASSIFY_SUBSTRATE
#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	uint GroupTileBitmask = 0;
#else
	SharedTileBitmask[LinearThreadIndex] = 0;

	GroupMemoryBarrierWithGroupSync();
#endif
#endif

	FLumenMaterialData Material = (FLumenMaterialData) 0;
	Material.SceneDepth = -1.0f;

	bool bIsValid = false;
	bool bIsAnyValid = false;

#if TILE_INDIRECT_DISPATCH
	// TileIndirectDispatchData stores view-relative tile coords.
	const uint PackedTileCoord = TileIndirectDispatchData[GroupId.x];
	GroupId = uint2(PackedTileCoord & 0xFFFFu, PackedTileCoord >> 16u);
	DispatchThreadId = GroupId * THREADGROUP_SIZE + GroupThreadId;
#endif
	const FLumenMaterialCoord Coord = GetLumenMaterialCoord(DispatchThreadId, GroupId, GroupThreadId, bIsValid, bIsAnyValid, true /* bAddMinRect*/);

	if (bIsValid)
	{
		Material = ReadMaterialData(Coord, MaxRoughnessToTrace);
	}

#if COPY_DEPTH_AND_NORMAL
	if (Coord.ClosureIndex == 0)
	{
		RWDepthTexture[Coord.SvPosition] = Material.DeviceZ;

		FNormalAndShadingInfo Info;
		Info.Normal = Material.WorldNormalForPositionBias;
		Info.bIsHair = Material.bIsHair;
		Info.bHasBackfaceDiffuse = Material.bHasBackfaceDiffuse;
		RWNormalTexture[Coord.SvPosition] = PackNormalAndShadingInfo(Info);

		#if SUBSTRATE_ENABLED && SUBSTRATE_GBUFFER_FORMAT==1
		RWClosureTexture[Coord.SvPosition] = Material.GetClosureCount();
		#endif
	}
#endif

	if (Coord.ClosureIndex == 0)
	{
	#if (STOCHASTIC_SAMPLE_OFFSET & STOCHASTIC_SAMPLE_OFFSET_2x2) != 0
		DownsampleDepthAndNormal(GroupId, GroupThreadId, Coord.SvPosition, Material, uint2(2, 2), RWDownsampledSceneDepth2x2, RWDownsampledWorldNormal2x2);
	#endif

	#if (STOCHASTIC_SAMPLE_OFFSET & STOCHASTIC_SAMPLE_OFFSET_2x1) != 0
		DownsampleDepthAndNormal(GroupId, GroupThreadId, Coord.SvPosition, Material, uint2(2, 1), RWDownsampledSceneDepth2x1, RWDownsampledWorldNormal2x1);
	#endif
	}

#if TILE_CLASSIFY_LUMEN
	if (bIsValid && IsValid(Material))
	{
		uint TileBitmask = 0;
		uint DiffuseIntegrationMethod = GetDiffuseIntegrationMethod(Material);
		const float DiffuseLerp = RoughReflectionsDiffuseLerp(Material);

		if (IsHair(Material))
		{
			TileBitmask |= LUMEN_TILE_BITMASK_GI_ALL;
		}
		else if (DiffuseIntegrationMethod == DIFFUSE_INTEGRATION_IMPORTANCE_SAMPLE_BRDF || DiffuseLerp < 1.0f)
		{
			TileBitmask |= LUMEN_TILE_BITMASK_GI_IMPORTANCE_SAMPLE_BRDF;
		}
		else
		{
			TileBitmask |= LUMEN_TILE_BITMASK_GI_SIMPLE_DIFFUSE;
		}

		Material = ApplySmoothBias(Material, false /*bTopLayerRoughness*/);
		if (NeedRayTracedReflections(Material.Roughness, Material))
		{
			TileBitmask |= LUMEN_TILE_BITMASK_REFLECTIONS;
		}

		// Pack into shared tile bitmask for 8x8 reduction
		WriteSharedTileBitmask(LinearThreadIndex, TileBitmask, TILE_BITMASK_INDEX_LUMEN);
	}
#endif

#if TILE_CLASSIFY_MEGALIGHTS
	if (bIsValid)
	{
		uint TileBitmask = 0;

		if (Material.IsMegaLightsValid())
		{
			if (Material.IsSimple())
			{
				TileBitmask |= MEGALIGHTS_TILE_BITMASK_SIMPLE;
			}
			else if (Material.IsSingle())
			{
				TileBitmask |= MEGALIGHTS_TILE_BITMASK_SINGLE;
			}
			else if (Material.IsComplexSpecial())
			{
				TileBitmask |= MEGALIGHTS_TILE_BITMASK_COMPLEX_SPECIAL;
			}
			else
			{
				TileBitmask |= MEGALIGHTS_TILE_BITMASK_COMPLEX;
			}
		}

		const uint EyeIndex = 0;
		const uint GridIndex = ComputeLightGridCellIndex(Coord.SvPosition - View.ViewRectMin.xy, Material.SceneDepth, EyeIndex);
		const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridIndex);

		if (CulledLightGridHeader.bHasRectLight)
		{
			TileBitmask |= MEGALIGHTS_TILE_BITMASK_RECT_LIGHT;
		}

		if (CulledLightGridHeader.bHasTexturedLight && EnableTexturedRectLights != 0)
		{
			TileBitmask |= MEGALIGHTS_TILE_BITMASK_TEXTURED_RECT_LIGHT;
		}

		// Pack into shared tile bitmask for 8x8 reduction
		WriteSharedTileBitmask(LinearThreadIndex, TileBitmask, TILE_BITMASK_INDEX_MEGALIGHTS);
	}
#endif

#if TILE_CLASSIFY_SUBSTRATE

	// Init primitive index
	if (DispatchThreadId.x < SUBSTRATE_TILE_TYPE_COUNT && DispatchThreadId.y == 0)
	{
		const uint TileType = DispatchThreadId.x;
		const uint IndexCountPerInstance = bRectPrimitive > 0 ? 4 : 6;
		TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(TileType) + 0] = IndexCountPerInstance;
	}

	if (Material.bIsValid)
	{
		uint TileBitmask = 0;
		#if SUBSTRATE_COMPLEXSPECIALPATH
		if (Material.bIsComplexSpecial)
		{
			TileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL;
		}
		else
		#endif
		if (Material.bIsComplex)
		{
			TileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEX;
		}
		else if (Material.bIsSingle)
		{
			TileBitmask |= SUBSTRATE_TILE_BITMASK_SINGLE;
		}
		else if (Material.bIsSimple)
		{
			TileBitmask |= SUBSTRATE_TILE_BITMASK_SIMPLE;
		}

		// Pack into shared tile bitmask for 8x8 reduction
		WriteSharedTileBitmask(LinearThreadIndex, TileBitmask, TILE_BITMASK_INDEX_SUBSTRATE);
	}
#endif

#if TILE_CLASSIFY_LUMEN || TILE_CLASSIFY_MEGALIGHTS || TILE_CLASSIFY_SUBSTRATE
#if PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	GroupTileBitmask = WaveActiveBitOr(GroupTileBitmask);

	if (WaveGetLaneCount() < THREADGROUP_SIZE * THREADGROUP_SIZE)
	{
		if (LinearThreadIndex == 0)
		{
			SharedTileBitmask = 0;
		}

		GroupMemoryBarrierWithGroupSync();
		
		if (WaveIsFirstLane())
		{
			uint Unused;
			InterlockedOr(SharedTileBitmask, GroupTileBitmask, Unused);
		}
		
		GroupMemoryBarrierWithGroupSync();
		
		GroupTileBitmask = SharedTileBitmask;
	}	
#else // PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
	GroupMemoryBarrierWithGroupSync();

	// GroupShared reduction
	if (LinearThreadIndex < 32)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 32];
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < 16)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 16];
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < 8)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 8];
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < 4)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 4];
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < 2)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 2];
	}

	GroupMemoryBarrierWithGroupSync();

	if (LinearThreadIndex < 1)
	{
		SharedTileBitmask[LinearThreadIndex] = SharedTileBitmask[LinearThreadIndex] | SharedTileBitmask[LinearThreadIndex + 1];
	}
#endif // PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS

	if (LinearThreadIndex == 0)
	{
		#if TILE_CLASSIFY_LUMEN
		if (bIsAnyValid)
		{
			RWLumenTileBitmask[int3(Coord.SvPosition / TILE_SIZE, Coord.ClosureIndex)] = ReadSharedTileBitmask(0, TILE_BITMASK_INDEX_LUMEN);
		}
		#endif

		#if TILE_CLASSIFY_MEGALIGHTS
		if (Coord.ClosureIndex == 0)
		{
			RWMegaLightsTileBitmask[Coord.SvPosition / TILE_SIZE] = ReadSharedTileBitmask(0, TILE_BITMASK_INDEX_MEGALIGHTS);
		}
		#endif

		#if TILE_CLASSIFY_SUBSTRATE
		{
			const uint TileBitmask = ReadSharedTileBitmask(0, TILE_BITMASK_INDEX_SUBSTRATE);
			if (TileBitmask != 0)
			{
				const uint2 TileCoord = (Coord.SvPosition.xy - uint2(View.ViewRectMin.xy)) >> TILE_SIZE_DIV_SHIFT;
				const uint EncodedTile = SubstratePackTile(TileCoord, TileEncoding);

				#if SUBSTRATE_COMPLEXSPECIALPATH
				if (TileBitmask & SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL)
				{
					uint WriteToIndex;
					InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL) + 1], 1, WriteToIndex);
					TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL) + WriteToIndex] = EncodedTile;
				}
				else 
				#endif
				if (TileBitmask & SUBSTRATE_TILE_BITMASK_COMPLEX)
				{
					uint WriteToIndex;
					InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_COMPLEX) + 1], 1, WriteToIndex);
					TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_COMPLEX) + WriteToIndex] = EncodedTile;
				}
				else if (TileBitmask & SUBSTRATE_TILE_BITMASK_SINGLE)
				{
					uint WriteToIndex;
					InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_SINGLE) + 1], 1, WriteToIndex);
					TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_SINGLE) + WriteToIndex] = EncodedTile;
				}
				else // (TileBitmask & SUBSTRATE_TILE_BITMASK_SIMPLE)
				{
					uint WriteToIndex;
					InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_SIMPLE) + 1], 1, WriteToIndex);
					TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_SIMPLE) + WriteToIndex] = EncodedTile;
				}
			}
		}
		#endif
	}
#endif // TILE_CLASSIFY_LUMEN || TILE_CLASSIFY_MEGALIGHTS || TILE_CLASSIFY_SUBSTRATE

#if REPROJECT_LUMEN || REPROJECT_MEGALIGHTS
	if (bIsValid)
	{
		uint2 ScreenCoord = Coord.SvPosition;

		FLumenPackedPixelData LumenPackedPixelData = (FLumenPackedPixelData)LUMEN_INVALID_PACKED_PIXEL_DATA;
		FMegaLightsPackedPixelData MegaLightsPackedPixelData = (FMegaLightsPackedPixelData)MEGALIGHTS_INVALID_PACKED_PIXEL_DATA;

		if (Material.IsValid())
		{
			LumenPackedPixelData.Packed = 0;
			LumenPackedPixelData.SetHasBackfaceDiffuse(Material.bHasBackfaceDiffuse);
			MegaLightsPackedPixelData.Packed = 0;

			float2 ScreenUV = (ScreenCoord + 0.5f) * View.BufferSizeAndInvSize.zw;
			float2 ScreenPosition = (ScreenUV - View.ScreenPositionScaleBias.wz) / View.ScreenPositionScaleBias.xy;
			float3 TranslatedWorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, Material.SceneDepth), Material.SceneDepth, 1), View.ScreenToTranslatedWorld).xyz;
			const float RandomScalar = BlueNoiseScalar(ScreenCoord, StochasticLightingStateFrameIndex);

			#if (STOCHASTIC_SAMPLE_OFFSET & STOCHASTIC_SAMPLE_OFFSET_2x2) != 0
			{
				float4 UpsampleWeights = ComputeUpsampleWeights(GroupThreadId, ScreenCoord, ScreenUV, ScreenPosition, TranslatedWorldPosition, Material, uint2(2, 2));

				const uint2 StochasticBilinearOffset = GetStochasticBilinearOffset(RandomScalar, UpsampleWeights);
				bool bCanReconstruct = dot(UpsampleWeights, 1.0f) >= 0.01f;

				bool bSetForLumen = true;
				bool bSetForMegaLights = true;
			#if STOCHASTIC_SAMPLE_OFFSET == (STOCHASTIC_SAMPLE_OFFSET_2x1 | STOCHASTIC_SAMPLE_OFFSET_2x2)
				bSetForLumen = LumenStochasticSampleMode == STOCHASTIC_SAMPLE_OFFSET_2x2;
				bSetForMegaLights = MegaLightsStochasticSampleMode == STOCHASTIC_SAMPLE_OFFSET_2x2;
			#endif
				if (bSetForLumen)
				{
					LumenPackedPixelData.SetStochasticSampleOffset(StochasticBilinearOffset, bCanReconstruct);
				}
				if (bSetForMegaLights)
				{
					MegaLightsPackedPixelData.SetStochasticSampleOffset(StochasticBilinearOffset, bCanReconstruct);
				}
			}
			#endif

			#if (STOCHASTIC_SAMPLE_OFFSET & STOCHASTIC_SAMPLE_OFFSET_2x1) != 0
			{
				float4 UpsampleWeights = ComputeUpsampleWeights(GroupThreadId, ScreenCoord, ScreenUV, ScreenPosition, TranslatedWorldPosition, Material, uint2(2, 1));

				const uint2 StochasticBilinearOffset = GetStochasticBilinearOffset(RandomScalar, UpsampleWeights);
				bool bCanReconstruct = dot(UpsampleWeights, 1.0f) >= 0.01f;

				bool bSetForLumen = true;
				bool bSetForMegaLights = true;
			#if STOCHASTIC_SAMPLE_OFFSET == (STOCHASTIC_SAMPLE_OFFSET_2x1 | STOCHASTIC_SAMPLE_OFFSET_2x2)
				bSetForLumen = LumenStochasticSampleMode == STOCHASTIC_SAMPLE_OFFSET_2x1;
				bSetForMegaLights = MegaLightsStochasticSampleMode == STOCHASTIC_SAMPLE_OFFSET_2x1;
			#endif
				if (bSetForLumen)
				{
					LumenPackedPixelData.SetStochasticSampleOffset(StochasticBilinearOffset, bCanReconstruct);
				}
				if (bSetForMegaLights)
				{
					MegaLightsPackedPixelData.SetStochasticSampleOffset(StochasticBilinearOffset, bCanReconstruct);
				}
			}
			#endif

			const float DeviceZ = Material.DeviceZ;

			#if FRONT_LAYER_TRANSLUCENCY
				// We don't have velocity for front layer translucency currently
				float3 HistoryScreenPosition = GetHistoryScreenPosition(ScreenPosition, DeviceZ, DeviceZ, /*EncodedVelocity*/ 0);		
			#else
				float3 HistoryScreenPosition = GetHistoryScreenPosition(ScreenPosition, ScreenUV, DeviceZ);
			#endif
			float2 HistoryScreenUV = HistoryScreenPosition.xy * HistoryScreenPositionScaleBias.xy + HistoryScreenPositionScaleBias.wz;

			bool bHistoryWasOnScreen = all(HistoryScreenUV >= HistoryUVMinMax.xy) && all(HistoryScreenUV <= HistoryUVMinMax.zw);
			float2 HistoryScreenCoord;
			half2 HistoryBilinearWeights;

			// Encode and write out HistoryScreenCoord with subpixel precision
			{
				HistoryScreenUV = clamp(HistoryScreenUV, HistoryGatherUVMinMax.xy, HistoryGatherUVMinMax.zw);
				HistoryScreenCoord = HistoryScreenUV * HistoryBufferSizeAndInvSize.xy - 0.5f;

				uint2 Encoded = EncodeHistoryScreenCoord(HistoryScreenCoord, HistorySubPixelGridSizeAndInvSize.xy);
				RWEncodedHistoryScreenCoord[ScreenCoord] = Encoded;
				
				float4 Decoded = DecodeHistoryScreenCoord(Encoded, HistoryScreenCoordDecodeShift, HistorySubPixelGridSizeAndInvSize.zw);
				HistoryScreenCoord = Decoded.xy;
				HistoryBilinearWeights = half2(Decoded.zw);
			}

			if (bHistoryWasOnScreen)
			{
				// Check 12 texels for a 12-tap Catmull-Rom approximation done in 5 bilinear taps
				//   C | C
				// C C | C C
				// ---------
				// C C | C C
				//   C | C

				// If any tap is invalid then fallback to a 2x2 bilinear filter with per tap weighting
				//     | 
				//   B | B
				// ---------
				//   B | B
				//     |

				// All of this is done using 4 gathers:
				//    0x | 1x
				// 0y 0z | 1y 1z
				// -------------
				// 2x 2y | 3x 3y
				//    2z | 3z

				float2 HistoryGatherUV = (HistoryScreenCoord + 1.0f) * HistoryBufferSizeAndInvSize.zw;
				float2 HistoryGatherUVQuad0 = (HistoryScreenCoord + float2(0.0f, 0.0f)) * HistoryBufferSizeAndInvSize.zw;
				float2 HistoryGatherUVQuad1 = (HistoryScreenCoord + float2(2.0f, 0.0f)) * HistoryBufferSizeAndInvSize.zw;
				float2 HistoryGatherUVQuad2 = (HistoryScreenCoord + float2(0.0f, 2.0f)) * HistoryBufferSizeAndInvSize.zw;
				float2 HistoryGatherUVQuad3 = (HistoryScreenCoord + float2(2.0f, 2.0f)) * HistoryBufferSizeAndInvSize.zw;

				// Lumen always uses 2x2 bilinear
				FHistoryTaps4 LumenTaps;
				LumenTaps.Init();

				// MegaLights uses Catmull-Rom and bilinear
				FHistoryTaps12 MegaLightsTaps;
				MegaLightsTaps.Init();

				// Depth test
				{
					const float3 V = normalize(-TranslatedWorldPosition);
					const float NoV = dot(Material.WorldNormalForPositionBias, V);

					// #TODO: try plane depth weighting again?
					// Raise the threshold at grazing angles to compensate for TAA jitter causing a depth mismatch dependent on the angle
					// This also introduces some ghosting around characters, needs a better solution
					float ReprojectedSceneDepth = ConvertFromDeviceZ(HistoryScreenPosition.z);
					float LumenDisocclusionDistanceThreshold = Material.bHasBackfaceDiffuse ? LumenHistoryDistanceThresholdForFoliage : LumenHistoryDistanceThreshold;
					LumenDisocclusionDistanceThreshold *= lerp(0.5f, 1.5f, RandomScalar);

					float LumenDepthTreshold = ReprojectedSceneDepth * LumenDisocclusionDistanceThreshold / lerp(0.1, 1.0, saturate(NoV));
					float MegaLightsDepthTreshold = ReprojectedSceneDepth * MegaLightsHistoryDistanceThreshold / lerp(0.1, 1.0, saturate(NoV));

					#if REPROJECT_MEGALIGHTS
					{
						// MegaLights needs a bicubic filter
						float3 HistorySampleSceneDepthQuad0 = ConvertFromDeviceZ4(DepthHistoryTexture.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad0).wzxy).yzw;
						float3 HistorySampleSceneDepthQuad1 = ConvertFromDeviceZ4(DepthHistoryTexture.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad1).wzxy).xzw;
						float3 HistorySampleSceneDepthQuad2 = ConvertFromDeviceZ4(DepthHistoryTexture.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad2).wzxy).xyw;
						float3 HistorySampleSceneDepthQuad3 = ConvertFromDeviceZ4(DepthHistoryTexture.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad3).wzxy).xyz;

						LumenTaps.SetBilinear(bool4(
							abs(HistorySampleSceneDepthQuad0.z - ReprojectedSceneDepth) < LumenDepthTreshold,
							abs(HistorySampleSceneDepthQuad1.y - ReprojectedSceneDepth) < LumenDepthTreshold,
							abs(HistorySampleSceneDepthQuad2.y - ReprojectedSceneDepth) < LumenDepthTreshold,
							abs(HistorySampleSceneDepthQuad3.x - ReprojectedSceneDepth) < LumenDepthTreshold));

						MegaLightsTaps.SetBicubic(
							abs(HistorySampleSceneDepthQuad0 - ReprojectedSceneDepth) < MegaLightsDepthTreshold,
							abs(HistorySampleSceneDepthQuad1 - ReprojectedSceneDepth) < MegaLightsDepthTreshold,
							abs(HistorySampleSceneDepthQuad2 - ReprojectedSceneDepth) < MegaLightsDepthTreshold,
							abs(HistorySampleSceneDepthQuad3 - ReprojectedSceneDepth) < MegaLightsDepthTreshold);
					}
					#else
					{
						// Lumen only uses a bilinear filter
						float4 HistorySampleSceneDepth4 = ConvertFromDeviceZ4(DepthHistoryTexture.GatherRed(GlobalPointClampedSampler, HistoryGatherUV).wzxy);

						LumenTaps.SetBilinear(abs(HistorySampleSceneDepth4 - ReprojectedSceneDepth) < LumenDepthTreshold);
					}
					#endif
				}

				// Normal test
				#if HISTORY_REJECT_BASED_ON_NORMAL
				{
					float4 NormalR = NormalAndShadingInfoHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUV).wzxy;
					float4 NormalG = NormalAndShadingInfoHistory.GatherGreen(GlobalPointClampedSampler, HistoryGatherUV).wzxy;
					float4 NormalB = NormalAndShadingInfoHistory.GatherBlue(GlobalPointClampedSampler, HistoryGatherUV).wzxy;

					const float3 HistoryNormal00 = UnpackNormalAndShadingInfo(float4(NormalR.x, NormalG.x, NormalB.x, 0)).Normal;
					const float3 HistoryNormal10 = UnpackNormalAndShadingInfo(float4(NormalR.y, NormalG.y, NormalB.y, 0)).Normal;
					const float3 HistoryNormal01 = UnpackNormalAndShadingInfo(float4(NormalR.z, NormalG.z, NormalB.z, 0)).Normal;
					const float3 HistoryNormal11 = UnpackNormalAndShadingInfo(float4(NormalR.w, NormalG.w, NormalB.w, 0)).Normal;

					LumenTaps.TestBilinear(bool4(
						dot(HistoryNormal00, Material.WorldNormalForPositionBias) >= LumenHistoryNormalCosThreshold,
						dot(HistoryNormal10, Material.WorldNormalForPositionBias) >= LumenHistoryNormalCosThreshold,
						dot(HistoryNormal01, Material.WorldNormalForPositionBias) >= LumenHistoryNormalCosThreshold,
						dot(HistoryNormal11, Material.WorldNormalForPositionBias) >= LumenHistoryNormalCosThreshold));
				}
				#endif

				// Material test
				// For front layer we don't do material based rejection since these checks are not relevant
				// and on top of that NormalAndShadingInfoHistory is actually the previous frame FrontLayerTranslucency.GBufferA
				// so the ShadingInfoHistory is not even available.
				// We use the raw FrontLayerTranslucency.GBufferA as history to be able to check validity (.w > 0) in VSM's GeneratePageFlagsFromPixels.  
				#if !FRONT_LAYER_TRANSLUCENCY
				{
					const float4 PackedW = NormalAndShadingInfoHistory.GatherAlpha(GlobalPointClampedSampler, HistoryGatherUV).wzxy;

					// Reject based on the foliage material flag (bHasBackfaceDiffuse)
					const bool4 bHasBackfaceDiffuse4 = bool4(
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.x)).bHasBackfaceDiffuse,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.y)).bHasBackfaceDiffuse,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.z)).bHasBackfaceDiffuse,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.w)).bHasBackfaceDiffuse);

					LumenTaps.TestBilinear(Material.bHasBackfaceDiffuse.xxxx == bHasBackfaceDiffuse4);

					// If shading info history is available, used it to only fetch compatible pixels.
					// For now only hair pixel are filtered out as their shading model is too different from the others
					const bool4 bIsHair4 = bool4(
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.x)).bIsHair,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.y)).bIsHair,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.z)).bIsHair,
						UnpackNormalAndShadingInfo(float4(0, 0, 0, PackedW.w)).bIsHair);

					// Cheaper bilinear test is fine here as a bit wrong samples on a bicubic filter edges don't cause large artifacts
					MegaLightsTaps.TestBilinear(Material.bIsHair.xxxx == bIsHair4.x);
				}
				#endif

                // History might have invalid lighting pixels. Only use history lighting if it has valid data.
                #if REPROJECT_MEGALIGHTS
                {
					uint3 HistoryFramesAccumulatedQuad0 = MegaLightsNumFramesAccumulatedHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad0).wzxy.yzw;
					uint3 HistoryFramesAccumulatedQuad1 = MegaLightsNumFramesAccumulatedHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad1).wzxy.xzw;
					uint3 HistoryFramesAccumulatedQuad2 = MegaLightsNumFramesAccumulatedHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad2).wzxy.xyw;
					uint3 HistoryFramesAccumulatedQuad3 = MegaLightsNumFramesAccumulatedHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUVQuad3).wzxy.xyz;

					MegaLightsTaps.TestBicubic(
						HistoryFramesAccumulatedQuad0 > 0,
						HistoryFramesAccumulatedQuad1 > 0,
						HistoryFramesAccumulatedQuad2 > 0,
						HistoryFramesAccumulatedQuad3 > 0);
				}
				#endif

				// Use Closure count as discriminant to avoid reusing history data of pixels with different material topology
				// Concretely, this removes haloing artifact at materials' boundary
				#if REPROJECT_MEGALIGHTS && USE_CLOSURECOUNT_HISTORY_REJECTION
				{
					const uint4 PackedClosureCount = ClosureHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUV).wzxy;
					MegaLightsTaps.TestBilinear(Material.ClosureCount == PackedClosureCount);
				}
				#endif

				#if PERMUTATION_OVERFLOW_TILE
				{
					const uint4 PackedClosureCount = ClosureHistory.GatherRed(GlobalPointClampedSampler, HistoryGatherUV).wzxy;
					LumenTaps.TestBilinear(Coord.ClosureIndex < PackedClosureCount);
				}
				#endif

				MegaLightsPackedPixelData.SetBicubicFootprintValid(MegaLightsTaps.IsBicubicValid());

				// Output a 2x2 bilinear history mask for ScreenProbeTemporalReprojectionCS and DenoiserTemporalCS
				{
					bool4 LumenTapValid4 = LumenTaps.GetBilinear();
					bool4 MegaLightsTapValid4 = MegaLightsTaps.GetBilinear();

					// Don't interpolate from low quality history taps, as this this overweight those taps after weight renormalization and cause artifacts
					half4 HistoryWeights = half4(
						(1 - HistoryBilinearWeights.y) * (1 - HistoryBilinearWeights.x),
						(1 - HistoryBilinearWeights.y) * HistoryBilinearWeights.x,
						HistoryBilinearWeights.y * (1 - HistoryBilinearWeights.x),
						HistoryBilinearWeights.y * HistoryBilinearWeights.x);

					LumenTapValid4 = and(LumenTapValid4, HistoryWeights > 0.01);
					MegaLightsTapValid4 = and(MegaLightsTapValid4, HistoryWeights > 0.01);

					LumenPackedPixelData.SetHistorySampleValidity(LumenTapValid4);
					MegaLightsPackedPixelData.SetHistorySampleValidity(MegaLightsTapValid4);
				}
			}
		}

	#if REPROJECT_LUMEN
		RWLumenPackedPixelData[uint3(ScreenCoord, Coord.ClosureIndex)] = LumenPackedPixelData.Packed;
	#endif
	#if REPROJECT_MEGALIGHTS
		if (Coord.ClosureIndex == 0)
		{
			RWMegaLightsPackedPixelData[ScreenCoord] = MegaLightsPackedPixelData.Packed;
		}
	#endif
	}
#endif
}