// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

uint StochasticLightingStateFrameIndex;

uint2 GetDownsampleJitter2x1(uint2 DownsampledScreenCoord)
{
	// Checkerboarding
	uint2 Jitter;
	Jitter.x = (DownsampledScreenCoord.y + StochasticLightingStateFrameIndex) % 2;
	Jitter.y = 0;
	return Jitter;
}

uint2 GetDownsampleJitter2x2(uint2 DownsampledScreenCoord)
{
	uint2 CellIndex = DownsampledScreenCoord % 2;
	uint LinearIndex = CellIndex.x + CellIndex.y * 2;
	LinearIndex = (LinearIndex + StochasticLightingStateFrameIndex) % 4;

	// 4-rooks sampling pattern
	uint2 Jitter;
	Jitter.x = LinearIndex & 0x02 ? 1 : 0;
	Jitter.y = LinearIndex & 0x01 ? 0 : 1;
	return Jitter;
}

uint2 GetStochasticBilinearOffset(float RandomScalar, float4 UpsampleWeights, int2 SampleOffsets[4])
{
	int2 StochasticBilinearOffset = 0;

	float4 PreSum;
	PreSum.x = UpsampleWeights.x;
	PreSum.y = PreSum.x + UpsampleWeights.y;
	PreSum.z = PreSum.y + UpsampleWeights.z;
	PreSum.w = PreSum.z + UpsampleWeights.w;

	RandomScalar *= PreSum.w;

	if (RandomScalar <= PreSum.x && UpsampleWeights.x > 0.0)
	{
		StochasticBilinearOffset = SampleOffsets[0];
	}
	else if (RandomScalar <= PreSum.y && UpsampleWeights.y > 0.0)
	{
		StochasticBilinearOffset = SampleOffsets[1];
	}
	else if (RandomScalar <= PreSum.z && UpsampleWeights.z > 0.0)
	{
		StochasticBilinearOffset = SampleOffsets[2];
	}
	else
	{
		StochasticBilinearOffset = SampleOffsets[3];
	}
	
	return StochasticBilinearOffset;
}

// Stochastic bilinear 2x2 filter
int2 GetStochasticBilinearOffset(float RandomScalar, float4 UpsampleWeights)
{
	int2 SampleOffsets[4];
	SampleOffsets[0] = int2(0, 0);
	SampleOffsets[1] = int2(1, 0);
	SampleOffsets[2] = int2(0, 1);
	SampleOffsets[3] = int2(1, 1);

	return GetStochasticBilinearOffset(RandomScalar, UpsampleWeights, SampleOffsets);
}

// Stochastic trilinear 2x2x2 filter
uint3 GetStochasticTrilinearOffset(float RandomScalar, float4 InterpolationWeights0, float4 InterpolationWeights1)
{
	float4 PreSum0;
	float4 PreSum1;
	PreSum0.x = InterpolationWeights0.x;
	PreSum0.y = PreSum0.x + InterpolationWeights0.y;
	PreSum0.z = PreSum0.y + InterpolationWeights0.z;
	PreSum0.w = PreSum0.z + InterpolationWeights0.w;
	PreSum1.x = PreSum0.w + InterpolationWeights1.x;
	PreSum1.y = PreSum1.x + InterpolationWeights1.y;
	PreSum1.z = PreSum1.y + InterpolationWeights1.z;
	PreSum1.w = PreSum1.z + InterpolationWeights1.w;

	RandomScalar *= PreSum1.w;

	uint3 StochasticTrilinearOffset = 0;

	if (RandomScalar <= PreSum0.x && InterpolationWeights0.x > 0.0)
	{
		StochasticTrilinearOffset = uint3(0, 0, 0);
	}
	else if (RandomScalar <= PreSum0.y && InterpolationWeights0.y > 0.0)
	{
		StochasticTrilinearOffset = uint3(1, 0, 0);
	}
	else if (RandomScalar <= PreSum0.z && InterpolationWeights0.z > 0.0)
	{
		StochasticTrilinearOffset = uint3(0, 1, 0);
	}
	else if (RandomScalar <= PreSum0.w && InterpolationWeights0.w > 0.0)
	{
		StochasticTrilinearOffset = uint3(1, 1, 0);
	}
	else if (RandomScalar <= PreSum1.x && InterpolationWeights1.x > 0.0)
	{
		StochasticTrilinearOffset = uint3(0, 0, 1);
	}
	else if (RandomScalar <= PreSum1.y && InterpolationWeights1.y > 0.0)
	{
		StochasticTrilinearOffset = uint3(1, 0, 1);
	}
	else if (RandomScalar <= PreSum1.z && InterpolationWeights1.z > 0.0)
	{
		StochasticTrilinearOffset = uint3(0, 1, 1);
	}
	else
	{
		StochasticTrilinearOffset = uint3(1, 1, 1);
	}

	return StochasticTrilinearOffset;
}
