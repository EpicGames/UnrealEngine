// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1
#include "../EyeAdaptationCommon.ush"
#include "../TonemapCommon.ush"

#define LUT_SCALING	1.05f
#define LUT_SIZE	32.f
Texture3D ColorGradingLUT;
SamplerState ColorGradingLUTSampler;

float3 ColorLookupTable(float3 LinearColor)
{
	//@todo - reuse with PostProcessTonemap.usf and handle LUTShaper
	float3 LUTEncodedColor = LinToLog( LinearColor + LogToLin( 0 ) );
	float3 UVW = LUTEncodedColor * ((LUT_SIZE - 1) / LUT_SIZE) + (0.5f / LUT_SIZE);
	return ColorGradingLUT.SampleLevel(ColorGradingLUTSampler, UVW, 0).rgb * LUT_SCALING;
}

uint Tonemap;

/**
 * Simplified tonemapper, which is useful for visualizations done after the main tonemapping pass, so that they can match colors.
 */
float3 VisualizeTonemap(float3 LinearColor)
{
	float3 OutColor = LinearColor;

	if (Tonemap != 0)
	{
		OutColor = ColorLookupTable(LinearColor * View.OneOverPreExposure * EyeAdaptationLookup());
	}
	else
	{
		OutColor = LinearToSrgbBranchless(LinearColor);
	}

	return OutColor;
}

/**
 * Drop pixels outside of a round visualization output view tile
 */
bool IsInsideVisualizeRect(int2 PixelCoord, int2 OutputViewSize, int2 OutputViewOffset)
{
	if (!all(PixelCoord < OutputViewSize))
	{
		return false;
	}

	// Don't clip full screen views
	if (any(OutputViewOffset > 0))
	{
		float Border = 12.0;
		float2 RectMin = Border;
		float2 RectMax = OutputViewSize - Border - 1.0;

		float2 Distance2 = max(max(RectMin - PixelCoord, PixelCoord - RectMax), 0.0);	
		return length(Distance2) - Border < 0.5;
	}

	return true;
}