// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

// The MIT License
// Copyright © 2021 Xavier Chermain (ICUBE), Simon Lucas(ICUBE), Basile Sauvage (ICUBE), Jean-Michel Dishler (ICUBE) and Carsten Dachsbacher (KIT)
// Permission is hereby granted, free of charge, to any person obtaining a copy of this software and associated documentation files (the "Software"), to deal in the Software without restriction, including without limitation the rights to use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies of the Software, and to permit persons to whom the Software is furnished to do so, subject to the following conditions: The above copyright notice and this permission notice shall be included in all copies or substantial portions of the Software. THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY, FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM, OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE SOFTWARE.

// Implementation of
// Real-Time Geometric Glint Anti-Aliasing with Normal Map Filtering
// 2021 Xavier Chermain (ICUBE), Simon Lucas(ICUBE), Basile Sauvage (ICUBE), Jean-Michel Dishler (ICUBE) and Carsten Dachsbacher (KIT)

// Updated to match Epic's Unreal Engine needs and standards.

#define InvSqrt2						0.707106f
#define MaxAnisotropy					4.0f

#define GlintSafeThreshold				0.000001f

// FGlintLutParams contains variables linked to the GlintShadingLUT
struct FGlintLutParams
{
	float Alpha;
	int N;
	int NLevels;
	float LevelBias;
	float LevelMin;
};

FGlintLutParams GetGlintParams()
{
	FGlintLutParams GlintLutParams;
	const float4 Params0 = View.GlintLUTParameters0;
	const float4 Params1 = View.GlintLUTParameters1;
	GlintLutParams.Alpha		= Params0.x;
	GlintLutParams.N			= asint(Params0.y);
	GlintLutParams.NLevels		= asint(Params0.z);
	GlintLutParams.LevelBias	= Params0.w;
	GlintLutParams.LevelMin		= Params1.x;
	return GlintLutParams;
}

#ifndef FORCE_DISABLE_GLINTS_AA
#define FORCE_DISABLE_GLINTS_AA 0
#endif

// GLINT AA must be skipped for some shaders who do not support ddx/ddy, such as ray tracing shaders.
// It is also skiped for Compute shader (MegaLights). (available on SM6.6 but not on the Metal API yet).
// The good news is that the visual difference is minimal.
#define USE_GLINT_AA (!RAYCALLABLESHADER && !RAYHITGROUPSHADER && !RAYGENSHADER && !RAYMISSSHADER && !FORCE_DISABLE_GLINTS_AA)

// Geometric Glint Anti-Aliasing parameters
#define GGAAKernelSize						0.5
#define GGAAFilter						1
#define UseHemisDerivatives				true

float3 SlopeToNormal(float2 Slope)
{
	float Norm = sqrt(1 + Slope.x * Slope.x + Slope.y * Slope.y);
	return float3(-Slope.x, -Slope.y, 1.) / Norm;
}

//=============================================================================
//============== Non axis aligned anisotropic Beckmann ========================
//=============================================================================
float non_axis_aligned_anisotropic_beckmann(float x, float y, float sigma_x, float sigma_y, float rho)
{
	float x_sqr = x * x;
	float y_sqr = y * y;
	float sigma_x_sqr = sigma_x * sigma_x;
	float sigma_y_sqr = sigma_y * sigma_y;

	float z = ((x_sqr / sigma_x_sqr) - ((2. * rho * x * y)
		/ (sigma_x * sigma_y)) + (y_sqr / sigma_y_sqr));
	return exp(-z / (2. * (1. - rho * rho)))
		/ (2. * PI * sigma_x * sigma_y * sqrt(1. - rho * rho));
}

//=============================================================================
//===================== Inverse error function ================================
//=============================================================================
float erfinv(float x)
{
	float w, p;
	w = -log((1.0 - x) * (1.0 + x));
	if (w < 5.000000) {
		w = w - 2.500000;
		p = 2.81022636e-08;
		p = 3.43273939e-07 + p * w;
		p = -3.5233877e-06 + p * w;
		p = -4.39150654e-06 + p * w;
		p = 0.00021858087 + p * w;
		p = -0.00125372503 + p * w;
		p = -0.00417768164 + p * w;
		p = 0.246640727 + p * w;
		p = 1.50140941 + p * w;
	}
	else
	{
		w = sqrt(w) - 3.000000;
		p = -0.000200214257;
		p = 0.000100950558 + p * w;
		p = 0.00134934322 + p * w;
		p = -0.00367342844 + p * w;
		p = 0.00573950773 + p * w;
		p = -0.0076224613 + p * w;
		p = 0.00943887047 + p * w;
		p = 1.00167406 + p * w;
		p = 2.83297682 + p * w;
	}
	return p * x;
}

//=============================================================================
//======================== Hash function ======================================
//======================== Inigo Quilez =======================================
//================ https://www.shadertoy.com/view/llGSzw ======================
//=============================================================================
float hashIQ(uint n)
{
	// integer hash copied from Hugo Elias
	n = (n << 13U) ^ n;
	n = n * (n * n * 15731U + 789221U) + 1376312589U;
	return float(n & 0x7fffffffU) / float(0x7fffffff);
}

//=============================================================================
//================== Pyramid size at LOD level ================================
//=============================================================================
int pyramidSize(FGlintLutParams GlintLutParams, int level)
{
	return int(pow(2., float(GlintLutParams.NLevels - 1 - level)));
}

//=============================================================================
//=============== Sampling from a normal distribution =========================
//=============================================================================
float sampleNormalDistribution(float U, float mu, float sigma)
{
	float x = sigma * 1.414213f * erfinv(2.0f * U - 1.0f) + mu;
	return x;
}

float2 sampleBivariateNormalDistribution(float2 u, float sigmaX, float sigmaY, float rho) 
{
	// Sample an original 2D centered unit-variance Gaussian
	float sampleXO = sampleNormalDistribution(u.x, 0., 1.);
	float sampleYO = sampleNormalDistribution(u.y, 0., 1.);
	float2 sampleO = float2(sampleXO, sampleYO);

	// Matrix M transformed the original unit-variance Gaussian
	// into a non-axis aligned Gaussian with standard deviations sigmaX and
	// sigmaY and correlation factor rho
	// See Chermain et al. 2021 for more information
	float M00 = sigmaX * sqrt(1. - rho * rho);
	float M01 = rho * sigmaX;
	float M10 = 0.;
	float M11 = sigmaY;

	// Transform the sample using the linear transformation M
	return float2(
		sampleO.x * M00 + sampleO.y * M01,
		sampleO.x * M10 + sampleO.y * M11);
}

//=============================================================================
//===================== Spatially-varying, multiscale, ========================
//=============== and transformed slope distribution function  ================
//=============================== Equation 4 ==================================
//=============================================================================
float P22_M(
	FGlintLutParams GlintLutParams,
	float2 slope_h, int l, int s0, int t0,
	float2 slope_dx, float2 slope_dy,
	float3 sigma_x_y_rho, float l_dist, 
	float MicrofacetRelativeArea)
{
	// Coherent index
	int twoToTheL = int(pow(2., float(l)));
	s0 *= twoToTheL;
	t0 *= twoToTheL;

	// Seed pseudo random generator
	uint rngSeed = uint(s0 + 1549 * t0);

	float uDensityRandomisation = hashIQ(rngSeed * 2171U);

	// Discard cells by using microfacet relative area
	float uMicrofacetRelativeArea = hashIQ(rngSeed * 13U);
	if (uMicrofacetRelativeArea > MicrofacetRelativeArea)
	{
		return 0.f;
	}

	// Fix density randomisation to 2 to have better appearance
	float densityRandomisation = 2.;

	// Sample a Gaussian to randomise the distribution LOD around the 
	// distribution level l_dist
	l_dist = sampleNormalDistribution(uDensityRandomisation, l_dist,
		densityRandomisation);

	l_dist = clamp(int(round(l_dist)), 0, GlintLutParams.NLevels);

	// Recover roughness and slope correlation factor
	float sigma_x = sigma_x_y_rho.x;
	float sigma_y = sigma_x_y_rho.y;
	float rho = sigma_x_y_rho.z;

	// If we are too far from the surface, the SDF is a gaussian.
	if (l_dist == GlintLutParams.NLevels) {
		return non_axis_aligned_anisotropic_beckmann(slope_h.x, slope_h.y,
			sigma_x, sigma_y, rho);
	}

	// Random rotations to remove glint alignment
	float uTheta = hashIQ(rngSeed);
	float theta = 2.0 * PI * uTheta;

	float cosTheta = cos(theta);
	float sinTheta = sin(theta);

	//=========================================================================
	//========= Linearly transformed isotropic Beckmann distribution ==========
	//=========================================================================

	float SIGMA_DICT = GlintLutParams.Alpha * InvSqrt2;
	float tmp1 = SIGMA_DICT / (sigma_x * sqrt(1. - rho * rho));
	float tmp2 =-SIGMA_DICT * rho / (sigma_y * sqrt(1. - rho * rho));
	float tmp3 = SIGMA_DICT / sigma_y;

	//=========================================================================
	//============================= Contribution 2 ============================
	//======================= Slope correlation factor ========================
	//============================== Equation 18 ==============================
	// Former inverse transformation matrix (Chermain et al. 2020) was: 
	// SIGMA_DICT * float2x2(	1. / sigma_x, 0.     ,
	//							0.          , 1. / sigma_y)
	//=========================================================================

	float2x2 invM = float2x2(tmp1, 0.,		// first column
							tmp2, tmp3);	// second column						// This order should be correct with the following matrix multiply

	//=========================================================================
	//========================== END Contribution 2 ===========================
	//=========================================================================

	// Apply random rotation
	float2x2 invR = float2x2(cosTheta, -sinTheta,	// first column
							sinTheta, cosTheta);	// second column
	invM = mul(invM, invR);

	// Get back to original space
	// Equation 5
	float2 slope_h_o = mul(slope_h, invM);

	// The SDF is an even function
	float2 abs_slope_h_o = float2(abs(slope_h_o.x), abs(slope_h_o.y));

	int distPerChannel = GlintLutParams.N / 3;
	float alpha_dist_isqrt2_4 = GlintLutParams.Alpha * InvSqrt2 * 4.f;

	// After 4 standard deviations, the SDF equals zero
	if (abs_slope_h_o.x > alpha_dist_isqrt2_4
		|| abs_slope_h_o.y > alpha_dist_isqrt2_4)
		return 0.f;

	float u1 = hashIQ(rngSeed * 16807U);
	float u2 = hashIQ(rngSeed * 48271U);

	int i = int(u1 * float(GlintLutParams.N));
	int j = int(u2 * float(GlintLutParams.N));

	// 3 distributions values in one texel
	int distIdxXOver3 = i / 3;
	int distIdxYOver3 = j / 3;

	float texCoordX = abs_slope_h_o.x / alpha_dist_isqrt2_4;
	float texCoordY = abs_slope_h_o.y / alpha_dist_isqrt2_4;

	// We also need to scale the derivatives,
	// as slope_h, to maintained coherence
	slope_dx = slope_dx / alpha_dist_isqrt2_4;
	slope_dy = slope_dy / alpha_dist_isqrt2_4;

	float3 P_20_o, P_02_o;
	//=========================================================================
	//=========================== Contribution 1 ==============================
	//================================ GGAA ===================================
	//=========================================================================
#if USE_GLINT_AA
	if (GGAAFilter)
	{
		// As for the distribution, we transform the filtering kernel
		float2 transformed_slope_dx = mul(slope_dx, invM);
		float2 transformed_slope_dy = mul(slope_dy, invM);

		// Scale the kernel by user parameter
		transformed_slope_dx *= GGAAKernelSize;
		transformed_slope_dy *= GGAAKernelSize;

		P_20_o = View.GlintTexture.SampleGrad(
			View.GlintSampler,
			float3(texCoordX, 0.5, l_dist * distPerChannel + distIdxXOver3),
			transformed_slope_dx.x,
			transformed_slope_dy.x).rgb;

		P_02_o = View.GlintTexture.SampleGrad(
			View.GlintSampler,
			float3(texCoordY, 0.5, l_dist * distPerChannel + distIdxYOver3),
			transformed_slope_dx.y,
			transformed_slope_dy.y).rgb;
	}
	//=========================================================================
	//========================= END Contribution 1 ============================
	//=========================================================================
	else // Without geometric glint anti-aliasing
#endif
	{
		P_20_o = View.GlintTexture.SampleLevel(
			View.GlintSampler,
			float3(texCoordX, 0.5, l_dist * distPerChannel + distIdxXOver3),
			0.0f).rgb;
		P_02_o = View.GlintTexture.SampleLevel(
			View.GlintSampler,
			float3(texCoordY, 0.5, l_dist * distPerChannel + distIdxYOver3),
			0.0f).rgb;
	}

	// Equation 15
	const int Idx20 = int(fmod(i, 3)); // i % 3; // int(mod(i, 3));
	const int Idx02 = int(fmod(j, 3)); // j % 3; // int(mod(j, 3));
	return P_20_o[Idx20] * P_02_o[Idx02] * determinant(invM);
}

//=============================================================================
//===================== P-SDF for a discrete LOD ==============================
//=============================================================================

// Most of this function is similar to pbrt-v3 EWA function,
// which itself is similar to Heckbert 1889 algorithm, 
// http://www.cs.cmu.edu/~ph/texfund/texfund.pdf, Section 3.5.9.
// Go through cells within the pixel footprint for a givin LOD
float P22__glint_discrete_LOD(
	FGlintLutParams GlintLutParams,
	int l, float2 slope_h, float2 st, float2 dst0,
	float2 dst1, float2 slope_dx, float2 slope_dy,
	float3 sigma_x_y_rho, float l_dist, float MicrofacetRelativeArea)
{
	// Convert surface coordinates to appropriate scale for level
	float pyrSize = pyramidSize(GlintLutParams, l);
	st[0] = st[0] * pyrSize - 0.5f;
	st[1] = st[1] * pyrSize - 0.5f;
	dst0[0] *= pyrSize;
	dst0[1] *= pyrSize;
	dst1[0] *= pyrSize;
	dst1[1] *= pyrSize;

	// Compute ellipse coefficients to bound filter region
	float A = dst0[1] * dst0[1] + dst1[1] * dst1[1] + 1.;
	float B = -2. * (dst0[0] * dst0[1] + dst1[0] * dst1[1]);
	float C = dst0[0] * dst0[0] + dst1[0] * dst1[0] + 1.;

	float invF = 1. / (A * C - B * B * 0.25f);
	A *= invF;
	B *= invF;
	C *= invF;

	// Compute the ellipse's bounding box in texture space
	float det = -B * B + 4 * A * C;
	float invDet = 1 / det;
	float uSqrt = sqrt(det * C), vSqrt = sqrt(A * det);
	int s0 = int(ceil(st[0] - 2. * invDet * uSqrt));
	int s1 = int(floor(st[0] + 2. * invDet * uSqrt));
	int t0 = int(ceil(st[1] - 2. * invDet * vSqrt));
	int t1 = int(floor(st[1] + 2. * invDet * vSqrt));

	// Scan over ellipse bound and compute quadratic equation
	float sum = 0.f;
	float sumWts = 0;
	int nbrOfIter = 0;
	const int SafeMaxIterationCount = 100;
	LOOP
	for (int it = t0; it <= t1; ++it)
	{
		float tt = it - st[1];
		LOOP
		for (int is = s0; is <= s1; ++is)
		{
			float ss = is - st[0];
			// Compute squared radius 
			// and filter SDF if inside ellipse
			float r2 = A * ss * ss + B * ss * tt + C * tt * tt;
			if (r2 < 1)
			{
				// Weighting function used in pbrt-v3 EWA function
				float alpha = 2;
				float W_P = exp(-alpha * r2) - exp(-alpha);
				sum += P22_M(
					GlintLutParams,
					slope_h, l, is, it,
					slope_dx, slope_dy,
					sigma_x_y_rho, l_dist, MicrofacetRelativeArea) * W_P;
				sumWts += W_P;
			}
			nbrOfIter++;
			// Guardrail (Extremely rare case.)
			if (nbrOfIter > SafeMaxIterationCount)
				break;
		}
		// Guardrail (Extremely rare case.)
		if (nbrOfIter > SafeMaxIterationCount)
			break;
	}
	return sum / max(GlintSafeThreshold, sumWts);
}

//=============================================================================
//======== Evaluation of the procedural physically based glinty BRDF ==========
//=============================================================================
float f_P(float3 wo, float3 wi, float3 sigmas_rho, float GlintDensity, float2 TexCoord, float2 TexCoordDDX, float2 TexCoordDDY, bool bIncludeGeometryTerm=false)
{
	if (wo.z <= 0.)
		return 0;
	if (wi.z <= 0.)
		return 0;

	float3 wh = normalize(wo + wi);
	if (wh.z <= 0.)
		return 0;

	// Local masking shadowing
	if (dot(wo, wh) <= 0. || dot(wi, wh) <= 0.)
		return 0;

	FGlintLutParams GlintLutParams = GetGlintParams();

	// Clamp tex.coord to [-1..1] in order to avoid incorrect LUT look up. A value > 1, means a pixel footprint larger than 1 pixel, 
	// which causes the glints dot to be stretched, causing implausible pattern.
	TexCoordDDX = clamp(TexCoordDDX, -1.f, 1.f);
	TexCoordDDY = clamp(TexCoordDDY, -1.f, 1.f);

	// Compute texture derivatives
	float2 dst0 = TexCoordDDX;
	float2 dst1 = TexCoordDDY;

	// Normal to slope
	float2 slope_h = float2(-wh.x / wh.z, -wh.y / wh.z);

	float2 slope_dx = 0;
	float2 slope_dy = 0;
#if USE_GLINT_AA
	if (UseHemisDerivatives)
	{
		// Derivatives in the projected hemispherical domain
		float2 projected_half_vector = float2(wh.x, wh.y);
		slope_dx = ddx(projected_half_vector);
		slope_dy = ddy(projected_half_vector);
	}
	else
	{
		slope_dx = ddx(slope_h);
		slope_dy = ddy(slope_h);
	}
#endif

	float D_P = 0;
	float P22_P = 0.;

	//=========================================================================
	// Similar to pbrt-v3 MIPMap::Lookup function, 
	// http://www.pbr-book.org/3ed-2018/Texture/Image_Texture.html#EllipticallyWeightedAverage

	// Compute ellipse minor and major axes
	float dst0LengthSquared = length(dst0);
	dst0LengthSquared *= dst0LengthSquared;
	float dst1LengthSquared = length(dst1);
	dst1LengthSquared *= dst1LengthSquared;

	if (dst0LengthSquared < dst1LengthSquared)
	{
		// Swap dst0 and dst1
		float2 tmp = dst0;
		dst0 = dst1;
		dst1 = tmp;
	}
	float majorLength = length(dst0);
	float minorLength = length(dst1);

	// Clamp ellipse eccentricity if too large
	if (minorLength * MaxAnisotropy < majorLength
		&& minorLength > 0.)
	{
		float scale = majorLength / (minorLength * MaxAnisotropy);
		dst1 *= scale;
		minorLength *= scale;
	}
	//=========================================================================

	// Without footprint -> no reflection
	if (minorLength == 0)
	{
		D_P = 0;
	}
	else
	{
		// Reparameterize the glint density [0..1] into a LogMicrofacetDensity and MicrofacetRelativeArea value.
		// LogMicrofacetDensity has a visible range of [~10..21]. With this we do the following remapping:
		// * Range in [10..21] remaps to the expressable density
		// * Range in [0..10] uses MicrofacetRelativeArea to fade progressively the glints.
		const float LogMicrofacetDensity = lerp(0.f, 21.f, saturate(GlintDensity));
		const float MicrofacetRelativeArea = saturate(LogMicrofacetDensity / 10.f);

		// Choose LOD
		float l = max(0.0, GlintLutParams.NLevels - 1. + log2(minorLength));
		int il = int(floor(l));

		float w = l - float(il);

		// Number of microfacets in a cell at level il
		float n_il = pow(2., float(2 * il - (2 * (GlintLutParams.NLevels - 1))));
		n_il *= exp(LogMicrofacetDensity);
		float LOD_dist_il = log(n_il) / 1.38629; // 2. * log(2) = 1.38629

		// Number of microfacets in a cell at level il + 1
		float n_ilp1 = pow(2., float(2 * (il + 1) - (2 * (GlintLutParams.NLevels - 1))));
		n_ilp1 *= exp(LogMicrofacetDensity);
		float LOD_dist_ilp1 = log(n_ilp1) / 1.38629; // 2. * log(2) = 1.38629

		float P22_P_il = non_axis_aligned_anisotropic_beckmann(slope_h.x, slope_h.y, sigmas_rho.x, sigmas_rho.y, sigmas_rho.z);
		float P22_P_ilp1 = P22_P_il;

		// Apply here level space bias and clamp to a minimum. This before we evaluate each glint level.
		LOD_dist_il   = clamp(LOD_dist_il   + GlintLutParams.LevelBias, 0, max(0, GlintLutParams.NLevels - GlintLutParams.LevelMin));
		LOD_dist_ilp1 = clamp(LOD_dist_ilp1 + GlintLutParams.LevelBias, 0, max(0, GlintLutParams.NLevels - GlintLutParams.LevelMin));

		bool opti = MicrofacetRelativeArea > 0.99;

		if (int(round(LOD_dist_il)) < GlintLutParams.NLevels || !opti)
		{
			P22_P_il = P22__glint_discrete_LOD(GlintLutParams, il, slope_h, TexCoord, dst0, dst1, slope_dx, slope_dy, sigmas_rho, LOD_dist_il, MicrofacetRelativeArea);
		}

		if (int(round(LOD_dist_ilp1)) < GlintLutParams.NLevels || !opti)
		{
			P22_P_ilp1 = P22__glint_discrete_LOD(GlintLutParams, il + 1, slope_h, TexCoord, dst0, dst1, slope_dx, slope_dy, sigmas_rho, LOD_dist_ilp1, MicrofacetRelativeArea);
		}

		P22_P = lerp(P22_P_il, P22_P_ilp1, w);

		D_P = P22_P / (wh.z * wh.z * wh.z * wh.z);
	}

//	// V-cavity masking shadowing
//	float G1wowh = min(1., 2. * wh.z * wo.z / dot(wo, wh));
//	float G1wiwh = min(1., 2. * wh.z * wi.z / dot(wi, wh));
//	float G = G1wowh * G1wiwh;																						// TODO check 
//
//	// Fresnel is set to one for simplicity
//	// but feel free to use "real" Fresnel term
//	float3 F = float3(1.0f, 1.0f, 1.0f);																			// TODO check
//
//	// (wi dot wg) is cancelled by
//	// the cosine weight in the rendering equation
//	return (F * G * D_P) / (4. * wo.z);

	float G = 1.f;
	if (bIncludeGeometryTerm)
	{
		float G1wowh = min(1., 2. * wh.z * wo.z / dot(wo, wh));
		float G1wiwh = min(1., 2. * wh.z * wi.z / dot(wi, wh));
		G = G1wowh * G1wiwh;																						// TODO check 
	}
	return G * D_P / (4. * wo.z);
}

// Glint Sampling

// Returns the ith 1D distribution at level l. It uses two random
// numbers: one for sampling the positive 1D distribution, the other to
// inverse the sign of the sampled value.
float Sample_P_1D(FGlintLutParams GlintLutParams, float u, int ij, int l)
{
	// The distribution equals 0 after 4 standard deviations.
	// Slope standard deviation = alpha / sqrt(2)

	// 3 distributions values in one texel
	int distIdxOver3 = ij / 3;
	int distPerChannel = GlintLutParams.N / 3;
	float alpha_dist_isqrt2_4 = GlintLutParams.Alpha * InvSqrt2 * 4.f;

	// The 1D distribution is an even function
	// [-1, 1]
	float SignScale = 1;
	if (u < 0.5)
	{
		SignScale *= -1.;
		u = RescaleRandomNumber(u, 0.0, 0.5);
	}
	else
	{
		u = RescaleRandomNumber(u, 0.5, 1.f);
	}

	// Equation 15
	const int Idx = int(fmod(ij, 3)); // ij % 3; // int(mod(ij, 3))

	// Slow version - the CDF is computed inline within the shader (TODO precompute CDF textures)
	uint it;
	const uint EntryCount = 64;
	float Cdfs[EntryCount+1];
	float Acc = 0;
	LOOP
	for (it=0;it<EntryCount;++it)
	{
		const float texCoordX = saturate(float(it) / float(EntryCount-1));
		float V = View.GlintTexture.SampleLevel(View.GlintSampler, float3(texCoordX, 0.5, l * distPerChannel + distIdxOver3), 0.0f)[Idx];

		Cdfs[it] = Acc;
		Acc += V;
	}
	Acc = max(0.0001f, Acc);

	LOOP
	for (it = 0; it < EntryCount; ++it)
	{
		Cdfs[it] /= Acc;
	}
	Cdfs[EntryCount] = 1.f;

	// Samples the positive part of the 1D distribution
	// Sampled value is in [0, 1)
	float Sample = 0;
	LOOP
	for (it = 0; it < EntryCount; ++it)
	{
		if (Cdfs[it] <= u && u <= Cdfs[it + 1])
		{
			const float s = (u - Cdfs[it]) / max(0.0001f, (Cdfs[it+1]-Cdfs[it]));
			Sample = saturate((it + s) / float(EntryCount));
			break;
		}
	}

	// Output is in [-alpha_dist_isqrt2_4, alpha_dist_isqrt2_4]
	return SignScale * Sample * alpha_dist_isqrt2_4;
}

//=============================================================================
//===================== Spatially-varying, multiscale, ========================
//=============== and transformed slope distribution function  ================
//=============================== Equation 4 ==================================
//=============================================================================
float2 Sample_P22_M(FGlintLutParams GlintLutParams, float2 u01, int l, int s0, int t0, float3 sigma_x_y_rho, float l_dist)
{
	// Coherent index
	int twoToTheL = int(pow(2., float(l)));
	s0 *= twoToTheL;
	t0 *= twoToTheL;

	// Seed pseudo random generator
	uint rngSeed = uint(s0 + 1549 * t0);

	float uDensityRandomisation = hashIQ(rngSeed * 2171U);

	// Fix density randomisation to 2 to have better appearance
	float densityRandomisation = 2.;

	// Sample a Gaussian to randomise the distribution LOD around the 
	// distribution level l_dist
	l_dist = sampleNormalDistribution(uDensityRandomisation, l_dist, densityRandomisation);
	l_dist = clamp(int(round(l_dist)), 0, GlintLutParams.NLevels);

	// Recover roughness and slope correlation factor
	float sigma_x = sigma_x_y_rho.x;
	float sigma_y = sigma_x_y_rho.y;
	float rho = sigma_x_y_rho.z;

	// If we are too far from the surface, the SDF is a gaussian.
	if (l_dist == GlintLutParams.NLevels) 
	{
		return sampleBivariateNormalDistribution(u01, sigma_x, sigma_y, rho);
	}

	// Random rotations to remove glint alignment
	float uTheta = hashIQ(rngSeed);
	float theta = 2.0 * PI * uTheta;

	float cosTheta = cos(theta);
	float sinTheta = sin(theta);

	// Linear transformation represented by matrix M
	float SIGMA_DICT = GlintLutParams.Alpha * InvSqrt2;
	float M00 = 1. / SIGMA_DICT * (sigma_x * sqrt(1. - rho * rho));
	float M01 = 1. / SIGMA_DICT * rho * sigma_y;
	float M10 = 0.;
	float M11 = 1. / SIGMA_DICT * sigma_y;

	float MR00 =  cosTheta * M00 + sinTheta * M01;
	float MR01 = -sinTheta * M00 + cosTheta * M01;
	float MR10 =  cosTheta * M10 + sinTheta * M11;
	float MR11 = -sinTheta * M10 + cosTheta * M11;

	float u1 = hashIQ(rngSeed * 16807U);
	float u2 = hashIQ(rngSeed * 48271U);

	int i = int(u1 * float(GlintLutParams.N));
	int j = int(u2 * float(GlintLutParams.N));

	int il_dist = l_dist;

	// Sample original (dictionary) slope space
	const float slopeHXO = Sample_P_1D(GlintLutParams, u01.x, i, il_dist);
	const float slopeHYO = Sample_P_1D(GlintLutParams, u01.y, j, il_dist);

	float2 slope_h = float2(
		slopeHXO * MR00 + slopeHYO * MR01,
		slopeHXO * MR10 + slopeHYO * MR11);

	return slope_h;
}

// Most of this function is similar to pbrt-v3 EWA function,
// which itself is similar to Heckbert 1889 algorithm, 
// http://www.cs.cmu.edu/~ph/texfund/texfund.pdf, Section 3.5.9.
// Go through cells within the pixel footprint for a givin LOD
float2 Sample_P22_LOD(
	FGlintLutParams GlintLutParams,
	float u0, float u1, float u2,
	int l, float2 st, 
	float2 dst0, float2 dst1, 
	float3 sigma_x_y_rho, 
	float l_dist)
{
	// Convert surface coordinates to appropriate scale for level
	float pyrSize = pyramidSize(GlintLutParams, l);
	st[0] = st[0] * pyrSize - 0.5f;
	st[1] = st[1] * pyrSize - 0.5f;
	dst0[0] *= pyrSize;
	dst0[1] *= pyrSize;
	dst1[0] *= pyrSize;
	dst1[1] *= pyrSize;

	// Compute ellipse coefficients to bound filter region
	float A = dst0[1] * dst0[1] + dst1[1] * dst1[1] + 1.;
	float B = -2. * (dst0[0] * dst0[1] + dst1[0] * dst1[1]);
	float C = dst0[0] * dst0[0] + dst1[0] * dst1[0] + 1.;

	float invF = 1. / (A * C - B * B * 0.25f);
	A *= invF;
	B *= invF;
	C *= invF;

	// Compute the ellipse's bounding box in texture space
	float det = -B * B + 4 * A * C;
	float invDet = 1 / det;
	float uSqrt = sqrt(det * C), vSqrt = sqrt(A * det);
	int s0 = int(ceil(st[0] - 2. * invDet * uSqrt));
	int s1 = int(floor(st[0] + 2. * invDet * uSqrt));
	int t0 = int(ceil(st[1] - 2. * invDet * vSqrt));
	int t1 = int(floor(st[1] + 2. * invDet * vSqrt));

	// The size is in [1-4]x[1-4]
	int SizeT = t1-t0;
	int SizeS = s1-s0;

	// Scan over ellipse bound and compute quadratic equation

	// 1. Count valid cells
	// Cell picking is done over valid cell rather actual EWA weight
	// The foot print is at max 4x4
	uint2 selected_st = 0;
	{
		#define MAX_CELL_COUNT 64
		float Cdfs[MAX_CELL_COUNT + 1];
		uint2 Indices[MAX_CELL_COUNT];
		float Acc = 0;
		uint ValidCellCount = 0;
		const int SafeMaxIterationCount = 100;

		// 1.1 Build CDF
		int nbrOfIter = 0;
		uint ValidCount = 0;
		LOOP
		for (int it = t0; it <= t1; ++it)
		{
			float tt = it - st[1];
			LOOP
			for (int is = s0; is <= s1; ++is)
			{
				float ss = is - st[0];
				// Compute squared radius 
				// and filter SDF if inside ellipse
				float r2 = A * ss * ss + B * ss * tt + C * tt * tt;
				if (r2 < 1)
				{
					const uint Index = min(ValidCellCount, MAX_CELL_COUNT);

					// Weighting function used in pbrt-v3 EWA function
					float alpha = 2;
					float W_P = exp(-alpha * r2) - exp(-alpha);
					Cdfs[Index] = Acc;
					Indices[Index] = uint2(is, it);
					Acc += W_P;
					++ValidCellCount;
				}
				nbrOfIter++;
				// Guardrail (Extremely rare case.)
				if (nbrOfIter > SafeMaxIterationCount)
					break;
			}
			// Guardrail (Extremely rare case.)
			if (nbrOfIter > SafeMaxIterationCount)
				break;
		}

		ValidCellCount = min(ValidCellCount, MAX_CELL_COUNT);

		// 1.2 Normalized CDF
		uint CellIt;
		LOOP
		for (CellIt = 0; CellIt < ValidCellCount; ++CellIt)
		{
			Cdfs[CellIt] /= max(Acc, 0.0001f);
		}
		Cdfs[ValidCellCount] = 1;

		// 1.3 Sample CDF
		LOOP
		for (CellIt = 0; CellIt < ValidCellCount; ++CellIt)
		{
			if (Cdfs[CellIt] <= u0 && u0 <= Cdfs[CellIt + 1])
			{
				selected_st = Indices[CellIt];
				break;
			}
		}
	}

	// 2. Sample valid cell and return slope_h
	return Sample_P22_M(GlintLutParams, float2(u0, u1), l, selected_st.x, selected_st.y, sigma_x_y_rho, l_dist);
}

float3 Sample_P(float3 u, float3 wo, float3 sigmas_rho, float LogMicrofacetDensity, float2 TexCoord, float2 TexCoordDDX, float2 TexCoordDDY)
{
	FGlintLutParams GlintLutParams = GetGlintParams();

	float U0 = u.x;
	float U1 = u.y;
	float U2 = u.z;

	// Compute texture derivatives
	float2 texCoord = TexCoord;
	float2 dst0 = TexCoordDDX;
	float2 dst1 = TexCoordDDY;

	float3 OutWm = float3(0,0,1); // z+
	if (wo.z <= 0.)
	{
		return OutWm;
	}

	float3 D_P = float3(0.0f, 0.0f, 0.0f);
	float P22_P = 0.;

	//=========================================================================
	// Similar to pbrt-v3 MIPMap::Lookup function, 
	// http://www.pbr-book.org/3ed-2018/Texture/Image_Texture.html#EllipticallyWeightedAverage

	// Compute ellipse minor and major axes
	float dst0LengthSquared = dot(dst0, dst0);
	float dst1LengthSquared = dot(dst1, dst1);

	if (dst0LengthSquared < dst1LengthSquared)
	{
		Swap(dst0, dst1);
	}
	float majorLength = length(dst0);
	float minorLength = length(dst1);

	// Clamp ellipse eccentricity if too large
	if (minorLength * MaxAnisotropy < majorLength && minorLength > 0.)
	{
		float scale = majorLength / (minorLength * MaxAnisotropy);
		dst1 *= scale;
		minorLength *= scale;
	}
	//=========================================================================
	
	// Compute LOD
	if (minorLength > 0)
	{
		const float l = max(0.0, GlintLutParams.NLevels - 1. + log2(minorLength));
		const int il = int(floor(l));
		const float wLOD = l - float(il);

		// Number of microfacets in a cell at level il
		float n_il = pow(2., float(2 * il - (2 * (GlintLutParams.NLevels - 1))));
		n_il *= exp(LogMicrofacetDensity);
		float LOD_dist_il = log(n_il) / 1.38629; // 2. * log(2) = 1.38629

		// Number of microfacets in a cell at level il + 1
		float n_ilp1 = pow(2., float(2 * (il + 1) - (2 * (GlintLutParams.NLevels - 1))));
		n_ilp1 *= exp(LogMicrofacetDensity);
		float LOD_dist_ilp1 = log(n_ilp1) / 1.38629; // 2. * log(2) = 1.38629

		// Apply here level space bias and clamp to a minimum. This before we evaluate each glint level.
		LOD_dist_il   = clamp(LOD_dist_il   + GlintLutParams.LevelBias, 0, max(0, GlintLutParams.NLevels - GlintLutParams.LevelMin));
		LOD_dist_ilp1 = clamp(LOD_dist_ilp1 + GlintLutParams.LevelBias, 0, max(0, GlintLutParams.NLevels - GlintLutParams.LevelMin));

		// 1. Select LOD (N or N+1)
		uint LOD = il;
		float LODDist = LOD_dist_il;
		if (U2 < wLOD)
		{
			U2 = RescaleRandomNumber(U2, 0.0, wLOD);
			LOD = il;
			LODDist = LOD_dist_il;
		}
		else
		{
			U2 = RescaleRandomNumber(U2, wLOD, 1.0);
			LOD = il + 1;
			LODDist = LOD_dist_il + 1;
		}

		// 2. Select cell and sample distribution
		if (int(round(LODDist)) < GlintLutParams.NLevels)
		{
			const float2 slope_h = Sample_P22_LOD(GlintLutParams, U0, U1, U2, LOD, texCoord, dst0, dst1, sigmas_rho, LODDist);

			// Build micro normal and its V-cavity dual
			const float3 Wm = SlopeToNormal(slope_h);
			const float3 WmP = float3(-Wm.x, -Wm.y, Wm.z);

			// Projected area of the other facet of the V-cavity
			const float dotWOWMP = max(dot(wo, WmP), 0);
			const float dotWOWM = max(dot(wo, Wm), 0);
			const float projectedAreaWMP = dotWOWMP / max(0.0001f, dotWOWM + dotWOWMP);

			// Samples a facet of the V-cavity
			if (U2 < projectedAreaWMP)
				OutWm = WmP;
			else
				OutWm = Wm;
		}
	}
	return OutWm;
}
