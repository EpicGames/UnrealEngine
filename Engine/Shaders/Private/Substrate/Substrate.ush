// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Sanity guard.
#ifndef SUBSTRATE_ENABLED
	#define SUBSTRATE_ENABLED 1
	#error SUBSTRATE_ENABLED needs to be defined
#endif

#if SUBSTRATE_ENABLED

#include "../ShadingEnergyConservation.ush"
#include "../BRDF.ush"
#include "../ColorSpace.ush"
#include "../Common.ush"
#include "../DeferredShadingCommon.ush"
#include "../ParticipatingMediaCommon.ush"
#include "../ThinFilmBSDF.ush"
#include "../MortonCode.ush"
#include "../OctahedralCommon.ush"
#include "../SubsurfaceProfileCommon.ush"
#include "../BurleyNormalizedSSSCommon.ush"
#include "../GammaCorrectionCommon.ush"
#include "/Engine/Shared/SubstrateDefinitions.h"
#include "/Engine/Private/Substrate/SubstrateStatisticalOperators.ush"
#include "/Engine/Private/Substrate/SubstrateSubsurface.ush"
#include "../SpecularProfileCommon.ush"
#include "SubstrateToonBSDF.ush"

#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#define SUBSTRATE_CLAMPED_CLOSURE_COUNT 1
#endif

// Substrate should never use real loop, that can result in too high register allocation in some compiler.
// Only a maximum unrolled loop according to the maximum slab/closures per pixel is preferable.
#define SUBSTRATE_UNROLL		UNROLL
#define SUBSTRATE_UNROLL_N(X)	UNROLL_N(X)

#if SUBSTRATE_CLAMPED_CLOSURE_COUNT == 1
	#define Substrate_for_unroll(X,Y,Z) X; if(Y)
#else
	#define Substrate_for_unroll(X,Y,Z) SUBSTRATE_UNROLL_N(SUBSTRATE_CLAMPED_CLOSURE_COUNT) for(X;Y;Z)
#endif

#define SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION (!COMPILER_FXC)

// During the base pass or forward rendering, shared local bases are simply available from registers.
// If a BSDF is loaded for processing from the Substrate buffer, normals are unpacked on demand to reduce VGPR pressure and increase occupancy.
#ifndef SUBSTRATE_INLINE_SHADING
#define SUBSTRATE_INLINE_SHADING 0
#endif

// Substrate SUBSTRATE_SHADING_QUALITY define the lighting accuarcy/approximation
// 0: approximation lighting
// 1: accurante lighting
#ifndef SUBSTRATE_SHADING_QUALITY
#define SUBSTRATE_SHADING_QUALITY 1
#endif

#ifndef SUBSTRATE_DEFERRED_SHADING
#define SUBSTRATE_DEFERRED_SHADING (SUBSTRATE_INLINE_SHADING == 0)
#endif 

// Allow Substrate tree to be built and process. 
// For certain shader, tree building & processing is not needed. For such shader, the tree are bypassed to fix shader compilation issue on certain platforms
#ifndef ALLOW_SUBSTRATE_TREE
#define ALLOW_SUBSTRATE_TREE 1
#endif

// During the unpacking of the Substrate material data, if a slab has some sub-surface scattering, its BaseColor/Specular value will be 
// patched/overriden with white albedo (and optionally no specular) for computing the incoming irradiance without any albedo information. 
// This is required by the post-process/screen-space sub-surface methods. This override can be opt-out. This is used during the classification 
// pass for writing out the correct base color value into the FSubstrateSubsurfaceData
#ifndef SUBSTRATE_SSS_MATERIAL_OVERRIDE
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 1
#endif

#ifndef MATERIAL_SHADINGMODEL_SINGLELAYERWATER
#define MATERIAL_SHADINGMODEL_SINGLELAYERWATER 0 
#endif

#ifndef MATERIAL_FULLY_ROUGH
#define MATERIAL_FULLY_ROUGH 0
#endif

#ifndef MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR
#define MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR 0
#endif

#ifndef SUBSTRATE_CLAMPED_CLOSURE_COUNT
#define SUBSTRATE_CLAMPED_CLOSURE_COUNT 0
#endif

#ifndef SUBSTRATE_USES_CONVERSION_FROM_LEGACY
#define SUBSTRATE_USES_CONVERSION_FROM_LEGACY 0
#endif

#ifndef MATERIALBLENDING_ANY_TRANSLUCENT
#define MATERIALBLENDING_ANY_TRANSLUCENT 0
#endif

// Substrate packed data is read either from a view SubstrateMaterial resource (Texture2DArray) or a RayTracingPayload/
#ifndef SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
#define SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE (!RAYCALLABLESHADER && !RAYHITGROUPSHADER && !RAYGENSHADER && !RAYMISSSHADER)
#endif

#ifndef SUBSTRATE_FASTPATH 
#define SUBSTRATE_FASTPATH 0
#endif

#ifndef SUBSTRATE_SINGLEPATH 
#define SUBSTRATE_SINGLEPATH 0
#endif

// Allow SUBSTRATE_FASTPATH and SUBSTRATE_SINGLEPATH to drive the actual material's complexity, rather than using the material configuration/header bits
#ifndef SUBSTRATE_USES_FORCED_COMPLEXITY
#define SUBSTRATE_USES_FORCED_COMPLEXITY 1
#endif

#define SUBSTRATE_COMPLEXPATH (!SUBSTRATE_FASTPATH && !SUBSTRATE_SINGLEPATH)

#ifndef SUBSTRATE_COMPLEXSPECIALPATH
#define SUBSTRATE_COMPLEXSPECIALPATH 0
#endif

// SUBSTRATE_COMPLEXSPECIALPATH is defined by the material or lighting shaders as they see fit.
// However, this should only be done for SUBSTRATE_COMPLEXPATH path.
#if SUBSTRATE_COMPLEXSPECIALPATH
#ifndef SUBSTRATE_COMPLEXPATH
#error SUBSTRATE_COMPLEXSPECIALPATH is defined while is not defined SUBSTRATE_COMPLEXPATH
#endif
#endif

#define SUBSTRATE_GLINTS_ENABLED (SUBSTRATE_COMPLEXSPECIALPATH && PLATFORM_ENABLES_SUBSTRATE_GLINTS)

// If MATERIAL_USES_SPECULAR_PROFILE is defined (i.e.: deferred: base pass, forward: base pass + lighting), use the definition to enable/disable specular profile. 
// Otherwise enable specular evaluation based on project settings to ensure correct evaluation in global shader
#ifdef MATERIAL_USES_SPECULAR_PROFILE
#define SUBSTRATE_SPECPROFILE_ENABLED (MATERIAL_USES_SPECULAR_PROFILE && PLATFORM_ENABLES_SUBSTRATE_SPECULAR_PROFILE)
#else
#define SUBSTRATE_SPECPROFILE_ENABLED (SUBSTRATE_COMPLEXPATH && PLATFORM_ENABLES_SUBSTRATE_SPECULAR_PROFILE)
#endif

#define SUBSTRATE_LEGACY_MATERIAL_APPLIES_FINAL_WEIGHT (!MATERIALBLENDING_MASKED && !MATERIALBLENDING_SOLID && SUBSTRATE_LEGACY_PREMULT_ALPHA_OVERRIDE==0 || SUBSTRATE_MATERIAL_EXPORT_FROM_TRANSLUCENT)

#define SUBSTRATE_INLINE_SINGLELAYERWATER (SUBSTRATE_ENABLED && MATERIAL_IS_SUBSTRATE && SUBSTRATE_INLINE_SHADING && MATERIAL_SHADINGMODEL_SINGLELAYERWATER)

#ifndef SUBSTRATE_OPAQUE_MATERIAL
#define SUBSTRATE_OPAQUE_MATERIAL 0
#endif

// In case we do not have any information about anisotropic material being supported or not, we enable that possibility to be able to call SubstrateReadGBufferBSDF correctly and get good information.
#ifndef SUPPORTS_ANISOTROPIC_MATERIALS
#define SUPPORTS_ANISOTROPIC_MATERIALS 1
#endif

// This is set for material writting to GBuffer for instance based on input connection from the material graph.
// However, when InternalGetSubstrateSlabBSDF is called from SubstrateReadGBufferBSDF (from many global shaders) we have to assume that we can read and setup anisotropic material so we use the project setting only.
#ifndef MATERIAL_USES_ANISOTROPY
#define MATERIAL_USES_ANISOTROPY SUPPORTS_ANISOTROPIC_MATERIALS
#endif

// Behind the scene, we do all the simple volumetric lighting & transmitance computations considering a slab of 100 centimeters = 1 meter.
// If the user specifies a different thickness to simulate varying appearance, we rescale the mean free path in order to not have to store thickness as part of the BSDF description.
#define SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM	100.0f
#define SUBSTRATE_SIMPLEVOLUME_THICKNESS_M		(SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER)

#define SUBSTRATE_EPSILON						1e-10f
#define SUBSTRATE_EYE_DEFAULT_F0				0.028f

///////////////////////////////////////////////////////////////////////////////
// Sub-surface scattering
//
//  Material configuration 
//  ----------------------
//  * Use Subsurface Diffusion (on Slab)
//  * Is Thin Surface (on Material)
//  * Top layer/Bottom layer (on Material topology)
//
//  Material Parameters (Slab)
//  --------------------------
//  * Diffuse Albedo
//  * SSS MFP
//  * SSS MFP scale
//  * SSS Phase aniso
//  * SSS modes 
//     * None
//     * Wrap
//     * Two-Sided Wrap
//     * Diffusion
//     * Diffusion Profile
//     * Simple Volume
//
//  SSS techniques (shading implementation)
//  ---------------------------------------
//  * None
//  * Simple Volume
//  * Wrap (ols Subsurface)
//  * Two-Sided Wrap (old Foliage)
//  * Diffusion
//  * Diffusion Profile
// 
//  Usable techniques for each configuration 
//
//  * Deferred case
// 	|---------------------------------------------|---------------------------------------------|
// 	| !IsThin                                     | IsThin                                      |
// 	|---------------------------------------------|---------------------------------------------|
// 	| !Bottom              | Bottom               | !Bottom              | Bottom               |
// 	|----------------------|----------------------|----------------------|----------------------|
//	| None                 | None                 | None                 | None                 |
//	| Simple Volume        | Simple Volume        | Simple Volume        | Simple Volume        |
//	|                      | Wrap                 |                      | Wrap                 |
//	|                      | Two-Sided Wrap       |                      | Two-Sided Wrap       |
//	|                      | Diffusion            |                      | Diffusion            |
//	|                      | Diffusion Profile    |                      | Diffusion Profile    |
//
//  * Forward case
// 	|---------------------------------------------|---------------------------------------------|
// 	| !IsThin                                     | IsThin                                      |
// 	|---------------------------------------------|---------------------------------------------|
// 	| !Bottom              | Bottom               | !Bottom              | Bottom               |
// 	|----------------------|----------------------|----------------------|----------------------|
//	| None                 | None                 | None                 | None                 |
//	| Simple Volume        | Simple Volume        | Simple Volume        | Simple Volume        |
//	|                      | Wrap                 |                      | Wrap                 |
//	|                      | Two-Sided Wrap       |                      | Two-Sided Wrap       |
//
// How are stored the MFP data?
// ---------------------------
//
//                          | MaterialData                                             | SSSData
//  ------------------------ ---------------------------------------------------------- -----------------
//  * Diffusion Profile     | ProfileID + Scale                                        | BaseColor + ProfileID + Scale
//  * Diffusion             | Rescaled MFP (11/11/10)                                  | BaseColor + MFP x Coverage
//  * Wrap (old Foliage)    | Rescaled MFP (11/11/10)                                  | BaseColor + Opacity
//                          | Rescaled MFP + Opacity (30bits + 7bits) - (LegacyLayout) | BaseColor + Opacity
//  * Wrap (ols Subsurface) | Rescaled MFP (11/11/10)                                  | BaseColor + Opacity
//                          | Rescaled MFP + Opacity (30bits + 7bits) - (LegacyLayout) | BaseColor + Opacity
//  * Simple Volume         | Rescaled MFP                                             | NoStorage
//
// Note: When using non-profile SSS diffusion, rescale MFP so that we take into account the slab thickness. 
//       This is done 'after' storing the SSSHeader/SSSData, as the original MFP needs to be preserved for correct post-process diffusion.
//       This rescaled MFP value is only used for transmission evaluation, not the diffusion

///////////////////////////////////////////////////////////////////////////////
// BSDFs representation

struct FSubstrateBSDF
{
	uint				State;			// Extra informations and other enabled BSDF features

	// SUBSTRATE_TODO pack the following
	int					OperatorIndex;
	float3				LuminanceWeightV;

	float				CoverageAboveAlongN;
	float3				TransmittanceAboveAlongN;	// The "pre-coverage" transmittance towards the top of the material along the normal and ignoring this BSDF, e.g. only accumulating matter above it.

	int					bIsBottom;
	int					bIsTop;

#if SUBSTRATE_INLINE_SHADING

	float				Coverage;		// The coverage of the material, as ratio of visible matter in this pixel.

	float3				Emissive;		// All bsdf can potientially have emissive. This is never written out to the Substrate gbuffer but instead accumulated in the emissive buffer during the base pass.
	float				ThicknessCm;	// This must be kept in order to be able to normalize the mean free path to comply with our volumetric material math all done in a normalised slab of medium.
	float3				TmpMFP;			// TmpMFP is used to store user specified MFP while we do not know if this Substrate BSDF is going to be using simple volume lighting. So we always store the MFP on the side to not override SSSPROFILE radius scale sharing the same register.

	float				TopLayerDataWeight;// The weight this BSDF contributes to the top layer (normal and roughness for SSR and SSAO for instance)
#endif

	// Water has too many parameters to be stores in the extra VGPRs below. So we extend this structure for the case where it is evaluated inline only.
	// This is fine because when water is used, it is the only BSDF that can be used and it is only used to do in line shading.
#if SUBSTRATE_INLINE_SINGLELAYERWATER
	float4 InlineVGPRs[3];
#endif

#if SUBSTRATE_COMPLEXSPECIALPATH	
	float4 VGPRs[7];
#elif SUBSTRATE_COMPLEXPATH
	float4 VGPRs[6];
#else
	float4 VGPRs[5];
#endif
	uint   Haziness;					// Compilers such as FXC have troubles when using asfloat and asuint to propagate correct haziness bitfield around through registers. So we store that data as an actual uint.


	void EnableSlabMFPComputationSpace();

	void EnableSlabBSDFSimpleVolumetric();

	void SubstrateSanitizeBSDF();

	void SubstrateSetBSDFRoughness(in float Roughness);

	void PackSingleOrComplexPathSlabBSDF(float Dither, inout uint OutData1, inout uint OutData2);
	void UnpackSingleOrComplexPathSlabBSDF(uint Data0, uint Data1);

	void PostProcessSlabBeforeLighting(bool bIsSubstrateOpaqueMaterial);

	void PostProcessBSDFBeforeLighting(bool bIsSubstrateOpaqueMaterial);

	// Return true if the BSDF has side SSS data (i.e., SSSHeader/SSSData)
	bool HasScatteringData();

	// Return true if the BSDF transmits and scatters light on the opposite side
	bool HasBackScattering();

	// Return true if the BSDF has SSS diffusion enabled
	bool HasSSSDiffusion();

	void SubstrateBakeF90ToReflectivity(in float NoV);

	void SubstrateBakeSpecularProfileToReflectivity(in float NoV);

	void SubstrateBakeGlintToReflectivity();

	void SubstrateBakeHazinessToSingleRoughness();
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Storage layouts
// ---------------
// 3 types of storages layout:
// * Complex: store many (mixed) BSDFs
// * Single : store a single BSDF (with all feature apart from anisotropy, because we sample from the toplayer data texture which stores normal but not the tangent)
// * Simple : store a single Slab BSDF (with no extra feature)
// 
// Header State (8bits):
// * MATERIALMODE (None/SlabSimple/SlabSingle/SlabComplex/SLW/Hair/Eye)
// * HASPRECSHADOWMASK			
// * ZEROPRECSHADOWMASK			
// * CASTCONTACTSHADOW			
// * HASDYNINDIRECTSHADOWCASTER
// * ISSINGLELAYERWATER			
// * HASSUBSURFACE				
// 
// Layouts:
// * LAYOUT 2 -------------------------------------------------------------------------------------------------------------------------------------------------------------------------
//  (Complex) HEADER        28 |  State | AO | BSDF Count | Basis count | Basis type | ComplexSpecialPath
//                             |  8       8    4            3             4            1
//            BASIS(0)      32 |  Normal: Oct 16/16 | Tangent: 11/11/10
//            BASIS(1)      32 |  Normal: Oct 16/16 | Tangent: 11/11/10
//                             |
//            STATE(0)      16 |  BSDF Type | Normal Id | Aniso | TopLayer | SSSType | GreyWeighV | Haze | F90 | Simple vol. | MFP Plugged | Fuzz | Thin
//                             |  3           2           1       1          2         1            1      1     1             1             1      1 
//            BSDF(0)       ?? |  Adaptive
//            STATE(1)      16 |  BSDF Type | Normal Id | Aniso | TopLayer | SSSType | GreyWeighV | Haze | F90 | Simple vol. | MFP Plugged | Fuzz | Thin
//                             |  3           2           1       1          2         1            1      1     1             1             1      1 
//            BSDF(1)       ?? |  Adaptive
//                             |
// * LAYOUT 1 -------------------------------------------------------------------------------------------------------------------------------------------------------------------------
//  (Single)  HEADER        30 |  State | AO | Opt Layout | Aniso | TopLayer | SSSType | GreyWeighV | Haze | F90 | Simple vol. | MFP Plugged | Fuzz | Thin
//                             |  8       8    3            1       1          2         1            1      1     1             1             1      1            
//            BASIS(0)       0 | (Use top layer normal)
//            BSDF(0)       ?? |  Adaptive
// * LAYOUT 0 -------------------------------------------------------------------------------------------------------------------------------------------------------------------------          
//  (Simple)  HEADER        32 | State | A0 | Roughness | DiffuseAbedo(8/20)
//   (Fast)                    | 8       8    8           8
//            BASIS(0)       0 | (Use top layer normal)
//            STATE(0)       0 | 
//            BSDF(0)       32 | F0_RGB | DiffuseAbedo(12/20)
//                             | 20       12
// * LAYOUT X -------------------------------------------------------------------------------------------------------------------------------------------------------------------------          
//   (Eye)    HEADER        32 | State | A0 | IrisMask | IrisMask
//                             | 8       8    8           8
//            BASIS(0)       0 | (Use top layer normal)
//            STATE(0)       0 | 
//            BSDF(0)       32 | DiffuseAlbedo | Roughness
//                             | 24              8
//                          32 | IrisNormal    | IrisPlaneNormal
//                             | 16              16
// * LAYOUT X -------------------------------------------------------------------------------------------------------------------------------------------------------------------------          
//   (Hair)   HEADER        32 | State | A0 | Free | Free
//                             | 8       8    8           8
//            BASIS(0)       0 | (Use top layer normal)
//            STATE(0)       0 | 
//            BSDF(0)       32 | BaseColor | Roughness
//                             | 24              8
//                          32 | Scatter | Specular | Backlit | ComplexTransmittance
//                             | 8         8          8         8
// * LAYOUT X -------------------------------------------------------------------------------------------------------------------------------------------------------------------------          
//   (SLW)    HEADER        32 | State | A0 | bUseSeparateDirLIight | Roughness | BaseColor0
//                             | 8       8    1           			7		  8
//            BASIS(0)       0 | (Use top layer normal)
//            STATE(0)       0 | 
//            BSDF(0)       32 | BaseColor1 | Metallic | Specular | TopMaterialOpacity
//                             | 12           8		  7		   7
//                          32 | OPTIONAL SeparatedMainDirLightLuminance if bUseSeparateDirLIight=true
//                             | 32 as R11G11B10F
//            -------------------------------------------------------------------------------------------------------------------------------------------------------------------------
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Bit utils functions
#define TO_BITMASK(BitCount)							((1u<<(BitCount))-1)
#define TO_OFFSET_BITMASK(BitCount, BitOffset)			(TO_BITMASK(BitCount)<<(BitOffset))
#define READ_BITS(X, BitCount, BitOffset)				uint((X>>(BitOffset)) & TO_BITMASK(BitCount))
#define WRITE_BITS(X, BitCount, BitOffset, NewValue)	X = (X & (~TO_OFFSET_BITMASK(BitCount, BitOffset))) | (((NewValue) & TO_BITMASK(BitCount)) << (BitOffset))

///////////////////////////////////////////////////////////////////////////////
// Material Header
//
// Bit count for each component stored in a Substrate header
// 
// State bit masks
#define HEADER_MASK_MATERIALMODE					(0x7)
#define HEADER_MASK_HASPRECSHADOWMASK				(1u << 3)
#define HEADER_MASK_ZEROPRECSHADOWMASK				(1u << 4)
#define HEADER_MASK_ISFIRSTPERSON					(1u << 4) // FirstPerson and ZEROPRECSHADOWMASK share the same bit, so the FirstPerson bit may only be used if ALLOW_STATIC_LIGHTING is 0.
#define HEADER_MASK_CASTCONTACTSHADOW				(1u << 5)
#define HEADER_MASK_HASDYNINDIRECTSHADOWCASTER		(1u << 6)
#define HEADER_MASK_HASSUBSURFACE					(1u << 7)
#define HEADER_MASK_BIT_COUNT						8

// For HEADER_BIT_BSDF_COUNT and  HEADER_BIT_SHAREDLOCALBASES_X, please refer to values in SubstrateDefinitions.h
#define HEADER_BIT_COUNT_STATE						HEADER_MASK_BIT_COUNT
#define HEADER_BIT_COUNT_IRRADIANCE_AO				8
// The remaining header bits are only available for the complex path.
#define HEADER_BIT_COUNT_BSDF_COUNT					4
#define HEADER_BIT_COUNT_SHAREDLOCALBASES_COUNT		3
#define HEADER_BIT_COUNT_SHAREDLOCALBASES_TYPE		4
#define HEADER_BIT_COUNT_COMPLEXPATH_MODE			2
#define HEADER_BIT_COUNT___UNUSED___				3
// End of header bits.

// When following the complex path, we can have complex slabs or toon shading BSDF.
// This is needed from the head to know how to unpack the incoming data.
#define COMPLEXPATH_MODE_SLAB						0
#define COMPLEXPATH_MODE_SLABSPECIAL				1
#define COMPLEXPATH_MODE_TOON						2
#define COMPLEXPATH_MODE_UNUSED						3		// Reserved for custom BSDF?

#define HEADER_BIT_OFFSET_STATE						0
#define HEADER_BIT_OFFSET_IRRADIANCE_AO				(HEADER_BIT_OFFSET_STATE					+ HEADER_BIT_COUNT_STATE)
#define HEADER_BIT_OFFSET_BSDF_COUNT				(HEADER_BIT_OFFSET_IRRADIANCE_AO			+ HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_BIT_OFFSET_SHAREDLOCALBASES_COUNT	(HEADER_BIT_OFFSET_BSDF_COUNT				+ HEADER_BIT_COUNT_BSDF_COUNT)
#define HEADER_BIT_OFFSET_SHAREDLOCALBASES_TYPE		(HEADER_BIT_OFFSET_SHAREDLOCALBASES_COUNT	+ HEADER_BIT_COUNT_SHAREDLOCALBASES_COUNT)
#define HEADER_BIT_OFFSET_COMPLEXPATH_MODE			(HEADER_BIT_OFFSET_SHAREDLOCALBASES_TYPE	+ HEADER_BIT_COUNT_SHAREDLOCALBASES_TYPE)

// Sanity check
#if (HEADER_BIT_OFFSET_COMPLEXPATH_MODE + HEADER_BIT_COUNT_COMPLEXPATH_MODE) > 32
#error Substrate Header is larger than 32 bits
#endif

// The different layout 'aliases' differently the material header
// 
// Layout 0 (Fast)
#define HEADER_FASTENCODING_BIT_COUNT				(HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_FASTENCODING_MASK					((1u<<HEADER_FASTENCODING_BIT_COUNT)-1u)
// Layout 1 (Single BSDF)
#define HEADER_SINGLEENCODING_BIT_COUNT				(HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_SINGLEENCODING_MASK					((1u<<HEADER_SINGLEENCODING_BIT_COUNT)-1u)
// Layout 2 (Complex)
#define HEADER_COMPLEXENCODING_BIT_COUNT			(32u)
#define HEADER_COMPLEXENCODING_MASK					(0xFFFFFFFFu)
// Layout Eye
#define HEADER_EYEENCODING_BIT_COUNT				(HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_EYEENCODING_MASK						((1u<<HEADER_EYEENCODING_BIT_COUNT)-1u)
// Layout Hair
#define HEADER_HAIRENCODING_BIT_COUNT				(HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_HAIRENCODING_MASK					((1u<<HEADER_HAIRENCODING_BIT_COUNT)-1u)
// Layout Eye
#define HEADER_SLWENCODING_BIT_COUNT				(HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO)
#define HEADER_SLWENCODING_MASK						((1u<<HEADER_SLWENCODING_BIT_COUNT)-1u)

// The single encoding bit count
#define HEADER_SINGLE_TOTAL_BIT_COUNT (HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT)

// A header always has a common state represented by stat bits (low 8 bits) and Irradiance/AO bits (higher 8 bits)
#define HEADER_GETCOMMONSTATES(X)					READ_BITS( X, (HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO), HEADER_BIT_OFFSET_STATE)
#define HEADER_SETCOMMONSTATES(X, NewValue)			WRITE_BITS(X, (HEADER_BIT_COUNT_STATE + HEADER_BIT_COUNT_IRRADIANCE_AO), HEADER_BIT_OFFSET_STATE, NewValue)

#define HEADER_GETSTATE(X)							READ_BITS( X, HEADER_BIT_COUNT_STATE, HEADER_BIT_OFFSET_STATE)
#define HEADER_SETSTATE(X, NewValue)				WRITE_BITS(X, HEADER_BIT_COUNT_STATE, HEADER_BIT_OFFSET_STATE, NewValue)

#define HEADER_GETIRRADIANCE_AO(X)					READ_BITS( X, HEADER_BIT_COUNT_IRRADIANCE_AO, HEADER_BIT_OFFSET_IRRADIANCE_AO)
#define HEADER_SETIRRADIANCE_AO(X, NewValue)		WRITE_BITS(X, HEADER_BIT_COUNT_IRRADIANCE_AO, HEADER_BIT_OFFSET_IRRADIANCE_AO, NewValue)

#define HEADER_GETBSDFCOUNT(X)						READ_BITS( X, HEADER_BIT_COUNT_BSDF_COUNT, HEADER_BIT_OFFSET_BSDF_COUNT)
#define HEADER_SETBSDFCOUNT(X, NewValue)			WRITE_BITS(X, HEADER_BIT_COUNT_BSDF_COUNT, HEADER_BIT_OFFSET_BSDF_COUNT, NewValue)

#define HEADER_GETSHAREDLOCALBASESCOUNT(X)			READ_BITS( X, HEADER_BIT_COUNT_SHAREDLOCALBASES_COUNT, HEADER_BIT_OFFSET_SHAREDLOCALBASES_COUNT)
#define HEADER_SETSHAREDLOCALBASESCOUNT(X, NewValue) WRITE_BITS(X, HEADER_BIT_COUNT_SHAREDLOCALBASES_COUNT, HEADER_BIT_OFFSET_SHAREDLOCALBASES_COUNT, NewValue)

#define HEADER_GETSHAREDLOCALBASESTYPE(X)			READ_BITS( X, HEADER_BIT_COUNT_SHAREDLOCALBASES_TYPE, HEADER_BIT_OFFSET_SHAREDLOCALBASES_TYPE)
#define HEADER_SETSHAREDLOCALBASESTYPE(X, NewValue)	WRITE_BITS(X, HEADER_BIT_COUNT_SHAREDLOCALBASES_TYPE, HEADER_BIT_OFFSET_SHAREDLOCALBASES_TYPE, NewValue)

#define HEADER_GETCOMPLEXPATHMODE(X)				READ_BITS( X, HEADER_BIT_COUNT_COMPLEXPATH_MODE, HEADER_BIT_OFFSET_COMPLEXPATH_MODE)
#define HEADER_SETCOMPLEXPATHMODE(X, NewValue)		WRITE_BITS(X, HEADER_BIT_COUNT_COMPLEXPATH_MODE, HEADER_BIT_OFFSET_COMPLEXPATH_MODE, NewValue)
#define HEADER_COMPLEXPATHMODE_MASK					TO_OFFSET_BITMASK(HEADER_BIT_COUNT_COMPLEXPATH_MODE, HEADER_BIT_OFFSET_COMPLEXPATH_MODE)

///////////////////////////////////////////////////////////////////////////////
// BSDF State
 
// Bit count for each component stored in a Substrate BSDF state
#define BSDF_BIT_COUNT_ANISOTROPY				1
#define BSDF_BIT_COUNT_TOPLAYER					1
#define BSDF_BIT_COUNT_SSSTYPE					3
#define BSDF_BIT_COUNT_GREYWEIGHT_V				1
#define BSDF_BIT_COUNT_HAZINESS					1
#define BSDF_BIT_COUNT_F90						1
#define BSDF_BIT_COUNT_MFPPLUGGED				1
#define BSDF_BIT_COUNT_HASFUZZ					1
#define BSDF_BIT_COUNT_ISTHIN					1
#define BSDF_BIT_COUNT_BSDF_TYPE				3
#define BSDF_BIT_COUNT_SHAREDLOCALBASESID		2
#define BSDF_BIT_COUNT_HASTRANSABOVE			1
#define BSDF_BIT_COUNT_HASGLINT					1
#define BSDF_BIT_COUNT_HASSPECPROFILE			1
#define BSDF_BIT_COUNT___UNUSED___				3
#define BSDF_BIT_COUNT_WEIGHT10F				10

// Bit offset of each component store in BSDF state 
#define BSDF_BIT_OFFSET_ANISOTROPY				0
#define BSDF_BIT_OFFSET_TOPLAYER				(BSDF_BIT_OFFSET_ANISOTROPY			+ BSDF_BIT_COUNT_ANISOTROPY)
#define BSDF_BIT_OFFSET_SSSTYPE					(BSDF_BIT_OFFSET_TOPLAYER			+ BSDF_BIT_COUNT_TOPLAYER)
#define BSDF_BIT_OFFSET_GREYWEIGHT_V			(BSDF_BIT_OFFSET_SSSTYPE			+ BSDF_BIT_COUNT_SSSTYPE)
#define BSDF_BIT_OFFSET_HAZINESS				(BSDF_BIT_OFFSET_GREYWEIGHT_V		+ BSDF_BIT_COUNT_GREYWEIGHT_V)
#define BSDF_BIT_OFFSET_F90						(BSDF_BIT_OFFSET_HAZINESS			+ BSDF_BIT_COUNT_HAZINESS)
#define BSDF_BIT_OFFSET_MFPPLUGGED				(BSDF_BIT_OFFSET_F90				+ BSDF_BIT_COUNT_F90)
#define BSDF_BIT_OFFSET_HASFUZZ					(BSDF_BIT_OFFSET_MFPPLUGGED			+ BSDF_BIT_COUNT_MFPPLUGGED)
#define BSDF_BIT_OFFSET_ISTHIN					(BSDF_BIT_OFFSET_HASFUZZ			+ BSDF_BIT_COUNT_HASFUZZ)
#define BSDF_BIT_OFFSET_BSDF_TYPE				(BSDF_BIT_OFFSET_ISTHIN				+ BSDF_BIT_COUNT_ISTHIN)
#define BSDF_BIT_OFFSET_SHAREDLOCALBASESID		(BSDF_BIT_OFFSET_BSDF_TYPE			+ BSDF_BIT_COUNT_BSDF_TYPE)
#define BSDF_BIT_OFFSET_HASTRANSABOVE			(BSDF_BIT_OFFSET_SHAREDLOCALBASESID	+ BSDF_BIT_COUNT_SHAREDLOCALBASESID)
#define BSDF_BIT_OFFSET_HASGLINT				(BSDF_BIT_OFFSET_HASTRANSABOVE		+ BSDF_BIT_COUNT_HASTRANSABOVE)
#define BSDF_BIT_OFFSET_HASSPECPROFILE			(BSDF_BIT_OFFSET_HASGLINT			+ BSDF_BIT_COUNT_HASGLINT)
#define BSDF_BIT_OFFSET_WEIGHT10F				22

// Layout 0 (Fast)
#define BSDF_FASTENCODING_BIT_COUNT				(0u)
#define BSDF_FASTENCODING_MASK					(0u)
// Layout 1 (Single BSDF)
#define BSDF_SINGLEENCODING_BIT_COUNT			(BSDF_BIT_OFFSET_ISTHIN + BSDF_BIT_COUNT_ISTHIN)
#define BSDF_SINGLEENCODING_MASK				((1u<<BSDF_SINGLEENCODING_BIT_COUNT)-1u)
// Layout 2 (Complex)
#define BSDF_COMPLEXCODING_BIT_COUNT			(32u)
#define BSDF_COMPLEXFASTENCODING_MASK			(0xFFFFFFFFu)

// Sanity check
#if (BSDF_BIT_OFFSET_WEIGHT10F + BSDF_BIT_COUNT_WEIGHT10F) > 32
#error Substrate BSDF state is larger than 32 bits
#endif
#if (HEADER_SINGLEENCODING_BIT_COUNT + BSDF_SINGLEENCODING_BIT_COUNT) > 32u
#error Substrate Single layout header is larger than 32 bits
#endif

// Accessors for the BSDF type
#define BSDF_GETTYPE(X)					READ_BITS(X.State,  BSDF_BIT_COUNT_BSDF_TYPE, BSDF_BIT_OFFSET_BSDF_TYPE)
#define BSDF_SETTYPE(X, NewType)		WRITE_BITS(X.State, BSDF_BIT_COUNT_BSDF_TYPE, BSDF_BIT_OFFSET_BSDF_TYPE, NewType)

// Accessors for shared local bases
// This assumes Normals[SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS]
#define BSDF_GETSHAREDLOCALBASISID(X)	READ_BITS(X.State,  BSDF_BIT_COUNT_SHAREDLOCALBASESID, BSDF_BIT_OFFSET_SHAREDLOCALBASESID)
#define BSDF_SETSHAREDLOCALBASISID(X, NrmlIdx)	WRITE_BITS(X.State, BSDF_BIT_COUNT_SHAREDLOCALBASESID, BSDF_BIT_OFFSET_SHAREDLOCALBASESID, NrmlIdx)

// Accessors for HasAnisotropy
// Indicates if the BSDF has anisotropic feature
#define BSDF_GETHASANISOTROPY(X)		READ_BITS(X.State,  BSDF_BIT_COUNT_ANISOTROPY, BSDF_BIT_OFFSET_ANISOTROPY)
#define BSDF_SETHASANISOTROPY(X, Aniso)	WRITE_BITS(X.State, BSDF_BIT_COUNT_ANISOTROPY, BSDF_BIT_OFFSET_ANISOTROPY, Aniso)

// Indicates if the BSDF is part of the top layer
#define BSDF_GETISTOPLAYER(X)			READ_BITS(X.State,  BSDF_BIT_COUNT_TOPLAYER, BSDF_BIT_OFFSET_TOPLAYER)
#define BSDF_SETISTOPLAYER(X, IsTop)	WRITE_BITS(X.State, BSDF_BIT_COUNT_TOPLAYER, BSDF_BIT_OFFSET_TOPLAYER, IsTop)

// Indicates if the BSDF has scattering component
#define BSDF_GETSSSTYPE(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_SSSTYPE, BSDF_BIT_OFFSET_SSSTYPE)
#define BSDF_SETSSSTYPE(X, Scatt)		WRITE_BITS(X.State, BSDF_BIT_COUNT_SSSTYPE, BSDF_BIT_OFFSET_SSSTYPE, Scatt)

// Indicates if the BSDF weight is grey scale so that it can be stored in a single float (e.g. top layer BSDFs)
#define BSDF_GETHASGREYWEIGHT_V(X)		READ_BITS(X.State,  BSDF_BIT_COUNT_GREYWEIGHT_V, BSDF_BIT_OFFSET_GREYWEIGHT_V)
#define BSDF_SETHASGREYWEIGHT_V(X, Grey)WRITE_BITS(X.State, BSDF_BIT_COUNT_GREYWEIGHT_V, BSDF_BIT_OFFSET_GREYWEIGHT_V, Grey)

// Indicates if the BSDF has haziness data
#define BSDF_GETHASHAZINESS(X)			READ_BITS(X.State,  BSDF_BIT_COUNT_HAZINESS, BSDF_BIT_OFFSET_HAZINESS)
#define BSDF_SETHASHAZINESS(X, Haze)	WRITE_BITS(X.State, BSDF_BIT_COUNT_HAZINESS, BSDF_BIT_OFFSET_HAZINESS, Haze)

// Indicates if the BSDF has edge color data
#define BSDF_GETHASF90(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_F90, BSDF_BIT_OFFSET_F90)
#define BSDF_SETHASF90(X, Col)			WRITE_BITS(X.State, BSDF_BIT_COUNT_F90, BSDF_BIT_OFFSET_F90, Col)

// Accessors for the 10bits float weight used if is it detected being a grey scale
#define BSDF_GETWEIGHT10F(X)			READ_BITS(X.State,  BSDF_BIT_COUNT_WEIGHT10F, BSDF_BIT_OFFSET_WEIGHT10F)
#define BSDF_SETWEIGHT10F(X, Weight)	WRITE_BITS(X.State, BSDF_BIT_COUNT_WEIGHT10F, BSDF_BIT_OFFSET_WEIGHT10F, Weight)

// Indicates if the BSDF has explicit MFP data
#define BSDF_GETHASMFP(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_MFPPLUGGED, BSDF_BIT_OFFSET_MFPPLUGGED)
#define BSDF_SETHASMFP(X, Value)		WRITE_BITS(X.State, BSDF_BIT_COUNT_MFPPLUGGED, BSDF_BIT_OFFSET_MFPPLUGGED, Value)

// Indicates if the BSDF has fuzz data
#define BSDF_GETHASFUZZ(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_HASFUZZ, BSDF_BIT_OFFSET_HASFUZZ)
#define BSDF_SETHASFUZZ(X, Value)		WRITE_BITS(X.State, BSDF_BIT_COUNT_HASFUZZ, BSDF_BIT_OFFSET_HASFUZZ, Value)

// Indicates if the BSDF is thin (e.g. not thick as a mesh but thin as a piece of paper, drives the shading model used)
#define BSDF_GETISTHIN(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_ISTHIN, BSDF_BIT_OFFSET_ISTHIN)
#define BSDF_SETISTHIN(X, Value)		WRITE_BITS(X.State, BSDF_BIT_COUNT_ISTHIN, BSDF_BIT_OFFSET_ISTHIN, Value)

// Indicates if the BSDF has color transmittance data (due to layers of matter above it)
#define BSDF_GETHASTRANSABOVE(X)		READ_BITS(X.State,  BSDF_BIT_COUNT_HASTRANSABOVE, BSDF_BIT_OFFSET_HASTRANSABOVE)
#define BSDF_SETHASTRANSABOVE(X, Value)	WRITE_BITS(X.State, BSDF_BIT_COUNT_HASTRANSABOVE, BSDF_BIT_OFFSET_HASTRANSABOVE, Value)

// Indicates if the BSDF has glint data
#define BSDF_GETHASGLINT(X)				READ_BITS(X.State,  BSDF_BIT_COUNT_HASGLINT, BSDF_BIT_OFFSET_HASGLINT) 
#define BSDF_SETHASGLINT(X, Value)		WRITE_BITS(X.State, BSDF_BIT_COUNT_HASGLINT, BSDF_BIT_OFFSET_HASGLINT, Value)

// Indicates if the BSDF has a specular profile
#define BSDF_GETHASSPECPROFILE(X)		READ_BITS(X.State,  BSDF_BIT_COUNT_HASSPECPROFILE, BSDF_BIT_OFFSET_HASSPECPROFILE) 
#define BSDF_SETHASSPECPROFILE(X, Value)WRITE_BITS(X.State, BSDF_BIT_COUNT_HASSPECPROFILE, BSDF_BIT_OFFSET_HASSPECPROFILE, Value)

#if SUBSTRATE_INLINE_SHADING
#define BSDF_SETEMISSIVE(X, V)			X.Emissive = V
#define BSDF_GETEMISSIVE(X)				X.Emissive
#else
#define BSDF_SETEMISSIVE(X, V)			
#define BSDF_GETEMISSIVE(X)				0.0f
#endif

#if SUBSTRATE_INLINE_SHADING
#define BSDF_SETTHICKNESSCM(X, V)		X.ThicknessCm = V
#define BSDF_GETTHICKNESSCM(X)			X.ThicknessCm
#else
#define BSDF_SETTHICKNESSCM(X, V)		
#define BSDF_GETTHICKNESSCM(X)			SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM
#endif

///////////////////////////////////////////////////////////////////////////////
// Slab BSDF members

#define SLAB_SINGLE_PATH_MASK  (TO_OFFSET_BITMASK(BSDF_BIT_COUNT_SSSTYPE,    BSDF_BIT_OFFSET_SSSTYPE)	| \
								TO_OFFSET_BITMASK(BSDF_BIT_COUNT_HAZINESS,   BSDF_BIT_OFFSET_HAZINESS)	| \
								TO_OFFSET_BITMASK(BSDF_BIT_COUNT_F90,        BSDF_BIT_OFFSET_F90)		| \
								TO_OFFSET_BITMASK(BSDF_BIT_COUNT_HASFUZZ,    BSDF_BIT_OFFSET_HASFUZZ)	| \
								TO_OFFSET_BITMASK(BSDF_BIT_COUNT_ISTHIN,     BSDF_BIT_OFFSET_ISTHIN))

#define SLAB_COMPLEX_PATH_MASK (TO_OFFSET_BITMASK(BSDF_BIT_COUNT_ANISOTROPY,       BSDF_BIT_OFFSET_ANISOTROPY) | \
								(SUBSTRATE_SPECPROFILE_ENABLED ? TO_OFFSET_BITMASK(BSDF_BIT_COUNT_HASSPECPROFILE,   BSDF_BIT_OFFSET_HASSPECPROFILE) : 0u))

#define SLAB_COMPLEX_SPECIAL_PATH_MASK ((SUBSTRATE_GLINTS_ENABLED ? TO_OFFSET_BITMASK(BSDF_BIT_COUNT_HASGLINT, BSDF_BIT_OFFSET_HASGLINT) : 0u))

#define SLAB_DIFFUSEALBEDO(X)			X.VGPRs[0].xyz
#define SLAB_ROUGHNESS(X)				X.VGPRs[0].w
#define SLAB_F0(X)						X.VGPRs[1].xyz
#define SLAB_ANISOTROPY(X)				X.VGPRs[1].w
#define SLAB_F90(X)						X.VGPRs[2].xyz
#define SLAB_HAZINESS(X)				X.Haziness
// Either with SSS profile
#define SLAB_SSSPROFILEID(X)			X.VGPRs[3].x
#define SLAB_SSSPROFILERADIUSSCALE(X)	X.VGPRs[3].y
#define SLAB_SSSPROFILETHICKNESSCM(X)	X.VGPRs[3].z
// Or explicit MFP in centimeter
#define SLAB_SSSMFP(X)					X.VGPRs[3].xyz
#define SLAB_SSSPHASEANISOTROPY(X)		X.VGPRs[3].w
// Fuzz to simulate cloth
#define SLAB_FUZZ_COLOR(X)				X.VGPRs[4].xyz
#define SLAB_FUZZ_AMOUNT(X)				X.VGPRs[4].w
#define SLAB_FUZZ_ROUGHNESS(X)			X.VGPRs[2].w
#if SUBSTRATE_COMPLEXPATH 
// Specular profile
#define SLAB_SPECPROFILEID(X)			X.VGPRs[5].w
#endif
#if SUBSTRATE_COMPLEXSPECIALPATH 
// Glints
#define SLAB_GLINT_UV(X)				X.VGPRs[5].xy
#define SLAB_GLINT_VALUE(X)				X.VGPRs[5].z
#define SLAB_GLINT_UVDDX(X)				X.VGPRs[6].xy
#define SLAB_GLINT_UVDDY(X)				X.VGPRs[6].zw
#endif

///////////////////////////////////////////////////////////////////////////////
// Volumetric BSDF members
#define VOLUMETRICFOGCLOUD_ALBEDO(X)	X.VGPRs[0].xyz
#define VOLUMETRICFOGCLOUD_EXTINCTION(X)X.VGPRs[1].xyz
#define VOLUMETRICFOGCLOUD_AO(X)		X.VGPRs[0].w

///////////////////////////////////////////////////////////////////////////////
// Unlit BSDF members
#define UNLIT_TRANSMITTANCE(X)			X.VGPRs[0].xyz
#define UNLIT_NORMAL(X)					X.VGPRs[1].xyz

///////////////////////////////////////////////////////////////////////////////
// Hair BSDF members
#define HAIR_BASECOLOR(X)				X.VGPRs[0].xyz
#define HAIR_ROUGHNESS(X)				X.VGPRs[0].w
#define HAIR_SCATTER(X)					X.VGPRs[1].x
#define HAIR_SPECULAR(X)				X.VGPRs[1].y
#define HAIR_BACKLIT(X)					X.VGPRs[1].z
#define HAIR_COMPLEXTRANSMITTANCE(X)	X.VGPRs[1].w

///////////////////////////////////////////////////////////////////////////////
// Eye BSDF members
#define EYE_DIFFUSEALBEDO(X)			X.VGPRs[0].xyz
#define EYE_ROUGHNESS(X)				X.VGPRs[0].w
#define EYE_F0(X)						X.VGPRs[1].xyz
#define EYE_F90(X)						X.VGPRs[2].xyz
#define EYE_IRISDISTANCE(X)				X.VGPRs[3].w
#define EYE_IRISMASK(X)					X.VGPRs[4].w
#define EYE_SSSPROFILEID(X)				X.VGPRs[1].w
#define EYE_IRISNORMAL(X)				X.VGPRs[3].xyz
#define EYE_IRISPLANENORMAL(X)			X.VGPRs[4].xyz

///////////////////////////////////////////////////////////////////////////////
// Single Layer Water BSDF members
#define SLW_BASECOLOR(X)				X.VGPRs[0].xyz
#define SLW_ROUGHNESS(X)				X.VGPRs[0].w
#define SLW_METALLIC(X)					X.VGPRs[1].x
#define SLW_SPECULAR(X)					X.VGPRs[1].y
#define SLW_TOPMATERIALOPACITY(X)		X.VGPRs[1].w
#if SUBSTRATE_INLINE_SINGLELAYERWATER
#define SLW_WATERALBEDO(X)				X.InlineVGPRs[0].xyz
#define SLW_WATEREXTINCTION(X)			X.InlineVGPRs[1].xyz
#define SLW_WATERPHASEG(X)				X.InlineVGPRs[1].w
#define SLW_COLORSCALEBEHINDWATER(X)	X.InlineVGPRs[2].xyz
#endif

///////////////////////////////////////////////////////////////////////////////
// Toon BSDF members
// TOON_PROFILEID is in [0, 255]
#define TOON_BASECOLOR(X)				X.VGPRs[0].xyz
#define TOON_PROFILEID(X)				X.VGPRs[0].w
#define TOON_METALLIC(X)				X.VGPRs[1].x
#define TOON_SPECULAR(X)				X.VGPRs[1].y
#define TOON_ROUGHNESS(X)				X.VGPRs[1].z
#define TOON_PATTERNUVS(X)				X.VGPRs[2].xy
#define TOON_ANISOTROPY(X)				X.VGPRs[2].z

///////////////////////////////////////////////////////////////////////////////
// Forward declarations

FParticipatingMedia SubstrateSlabCreateParticipatingMedia(float3 DiffuseColor, float3 MeanFreePathCentimeters);
float3 RescaleMFPToComputationSpace(float3 InMFPInCm, float InSrcThicknessInCm, float InDstThicknessInCm);
void   SubstrateRequestSharedLocalBasisTangent(inout uint Types, uint Index);
float3 SubstrateUnpackNormal(uint PackedNormal);
void   SubstrateUnpackNormalAndTangent(inout float3 Normal, inout float3 Tangent, in uint InPacked);
uint   PackR6(float Value, float Dither);
float  UnpackR6(uint Value);
uint   PackR7(float Value, float Dither);
float  UnpackR7(uint Value);
uint   PackR7G7B6Gamma2(float3 rgb, float Dither);
float3 UnpackR7G7B6Gamma2(uint rgb);
uint   PackColorLinearToGamma2(float3 In);
float3 UnpackColorGamma2ToLinear(uint In);
uint   PackColorLinearToGamma2AlphaLinear(float4 In);
float4 UnpackColorGamma2ToLinearAlphaLinear(uint In);
uint   SubstratePackNormal(in float3 Normal);
float3 SubstrateUnpackNormal(uint PackedNormal);
uint   SubstratePackNormal24(in float3 Normal);
float3 SubstrateUnpackNormal24(uint PackedNormal);
uint   SubstratePackNormal22(in float3 Normal);
float3 SubstrateUnpackNormal22(uint PackedNormal);

bool   IsSubstrateSlabCompatible_SimplePath(in FSubstrateBSDF BSDF);
bool   IsSubstrateSlabCompatible_SinglePath(in FSubstrateBSDF BSDF);
bool   IsSubstrateSlabCompatible_ComplexPath(in FSubstrateBSDF BSDF);
bool   IsSubstrateSlabCompatible_ComplexSpecialPath(in FSubstrateBSDF BSDF);

///////////////////////////////////////////////////////////////////////////////
// Helpers

bool SubstrateIsComplexMaterial(uint InShadingModelID, bool bHasAnisotropy)
{
	return bHasAnisotropy || InShadingModelID == SHADINGMODELID_EYE || InShadingModelID == SHADINGMODELID_HAIR || InShadingModelID == SHADINGMODELID_SUBSTRATE_TOON;
}

///////////////////////////////////////////////////////////////////////////////
// Haziness

#define DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT float2(0.0, 0.0)

// bSimpleClearCoat==false means that Weight is the blend in amount of the secondary specular lob over the first specular lob.
// bSimpleClearCoat==true  means fast clear coat is used, Weight is coverage of a top layer specular. 
struct FHaziness
{
	bool  bSimpleClearCoat;
	float Roughness;
	float Weight;
	float2 BottomNormalOct;	// 2 x 8 bits

	bool HasBottomNormal()
	{
		return CLEAR_COAT_BOTTOM_NORMAL && any(BottomNormalOct != DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT);
	}
};

float3 SubstrateGetSimpleCoatBottomNormal(float2 BottomNormalOct, float3 TopWorldNormal)
{
	const float2 OctNormal = ((BottomNormalOct * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(TopWorldNormal);
	return OctahedronToUnitVector(OctNormal);
}

float2 UnpackBottomNormalOct(uint BottomNormalOct16bits)
{
	return float2(UnpackR8((BottomNormalOct16bits >> 0) & 0xFF), UnpackR8((BottomNormalOct16bits >> 8) & 0xFF));
}

FHaziness InitialiseHaziness()
{
	FHaziness Haziness = (FHaziness)0;
	return Haziness;
}

// returns 32 bits: 1 bit bFastClearCoat, 7 bits Weight, 8 bits roughness, 16bits coat oct bottom normal (only used when CLEAR_COAT_BOTTOM_NORMAL)
uint PackHaziness(FHaziness Haziness)
{
	uint High8Bits = (Haziness.bSimpleClearCoat ? 0x80 : 0x00) | uint(saturate(Haziness.Weight) * 127);
	uint Data = (High8Bits << 8) | PackR8(Haziness.Roughness);
#if CLEAR_COAT_BOTTOM_NORMAL
	Data |= (PackR8(Haziness.BottomNormalOct.x) << 16) | (PackR8(Haziness.BottomNormalOct.y) << 24);
#endif
	return Data;
}

FHaziness UnpackHaziness(uint Data)
{
	FHaziness Haziness;
	Haziness.bSimpleClearCoat	= Data & 0x8000;
	Haziness.Weight				= float(uint((Data>>8) & 0x7F)) * (1.0f / 127.0f);
	Haziness.Roughness			= UnpackR8(Data);
#if CLEAR_COAT_BOTTOM_NORMAL
	Haziness.BottomNormalOct = UnpackBottomNormalOct(Data >> 16);
#else
	Haziness.BottomNormalOct = DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT;
#endif
	return Haziness;
}

float GetHazinessRoughness(uint Data)
{
	return UnpackR8(BitFieldExtractU32(Data, 8, 0));
}

uint SetHazinessRoughness(uint Data, float InRoughness)
{
	return (Data & 0xFFFFFF00) | PackR8(InRoughness);
}

uint LerpHazinessParameterBlending(FSubstrateBSDF A, FSubstrateBSDF B, float MixValue)
{
	FHaziness HazinessA = UnpackHaziness(SLAB_HAZINESS(A));
	FHaziness HazinessB = UnpackHaziness(SLAB_HAZINESS(B));

	FHaziness OutputHaziness;
	OutputHaziness.Weight			= lerp(HazinessA.Weight,	HazinessB.Weight,		MixValue);
	OutputHaziness.Roughness		= lerp(HazinessA.Roughness, HazinessB.Roughness,	MixValue);
	OutputHaziness.bSimpleClearCoat = HazinessA.bSimpleClearCoat || HazinessB.bSimpleClearCoat;					// Maintain the fast legacy path first
#if CLEAR_COAT_BOTTOM_NORMAL
	OutputHaziness.BottomNormalOct = lerp(HazinessA.BottomNormalOct, HazinessB.BottomNormalOct, MixValue.xx);	// Approximation
#endif

	return PackHaziness(OutputHaziness);
}

///////////////////////////////////////////////////////////////////////////////
// Irradiance and occlusion
struct FSubstrateIrradianceAndOcclusion
{
	float MaterialAO;
	float IndirectIrradiance;
	uint  DiffuseIndirectSampleOcclusion;
};

FSubstrateIrradianceAndOcclusion InitIrradianceAndOcclusion(float InAO=1.0f)
{
	FSubstrateIrradianceAndOcclusion Out;
	Out.MaterialAO = 1.0f;
	Out.IndirectIrradiance = 1.0f;
	Out.DiffuseIndirectSampleOcclusion = 0x0u;
	return Out;
}

uint SubstratePackIrradianceAndOcclusion(FSubstrateIrradianceAndOcclusion In, float QuantizationBias=0)
{
#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
	return In.DiffuseIndirectSampleOcclusion;
#elif ALLOW_STATIC_LIGHTING
	// No space for AO. Multiply IndirectIrradiance by AO instead of storing.
	return min(0xFFu, (EncodeIndirectIrradiance(In.IndirectIrradiance * In.MaterialAO) + QuantizationBias * (1.0 / 255.0)) * 0xFFu);
#else
	return PackR8(In.MaterialAO);
#endif
}

FSubstrateIrradianceAndOcclusion SubstrateUnpackIrradianceAndOcclusion(uint In /*8bits*/)
{
	FSubstrateIrradianceAndOcclusion Out;
#if GBUFFER_HAS_DIFFUSE_SAMPLE_OCCLUSION
	Out.DiffuseIndirectSampleOcclusion = In;
	Out.MaterialAO = saturate(1.0 - float(countbits(Out.DiffuseIndirectSampleOcclusion)) * rcp(float(INDIRECT_SAMPLE_COUNT)));
	Out.IndirectIrradiance = 1.f;
#elif ALLOW_STATIC_LIGHTING
	Out.MaterialAO = 1.f;
	Out.DiffuseIndirectSampleOcclusion = 0x0u;
	Out.IndirectIrradiance = DecodeIndirectIrradiance(UnpackR8(In));
#else
	Out.MaterialAO = float(In)* rcp(255.0f);
	Out.DiffuseIndirectSampleOcclusion = 0x0u;
	Out.IndirectIrradiance = 1.f;
#endif
	return Out;
}

FSubstrateIrradianceAndOcclusion SubstrateGetIrradianceAndAO(FGBufferData In)
{	
	FSubstrateIrradianceAndOcclusion Out = InitIrradianceAndOcclusion();
	Out.MaterialAO 						= In.GBufferAO;
	Out.DiffuseIndirectSampleOcclusion 	= In.DiffuseIndirectSampleOcclusion;
	Out.IndirectIrradiance 				= In.IndirectIrradiance;
	return Out;
}

///////////////////////////////////////////////////////////////////////////////
// Top layer

#define TOP_LAYER_MATERIAL_NONE		0
#define TOP_LAYER_MATERIAL_VALID	1
#define TOP_LAYER_MATERIAL_SLWATER	2
// Available #define TOP_LAYER_MATERIAL_xxx	3

struct FSubstrateTopLayerData
{
	float3 UnlitViewBaseColor; // This data is only available during the base pass. It is not packed/stored otherwise.
	float3 WorldNormal;
	float  Roughness;
	uint   Material;
};

SUBSTRATE_TOP_LAYER_TYPE SubstratePackTopLayerData(FSubstrateTopLayerData In)
{
#if SUBSTRATE_NORMAL_QUALITY==0
	// Output is min. by 1 to flag data has valid. This creates a small bias for one particular orientation with a mirror roughness.
	return (SubstratePackNormal22(In.WorldNormal) << 10) | (PackR8(In.Roughness) << 2) | (In.Material);
#elif SUBSTRATE_NORMAL_QUALITY==1
	return uint2(SubstratePackNormal(In.WorldNormal), (PackR8(In.Roughness) << 2) | (In.Material));
#else
#error Unkown SUBSTRATE_NORMAL_QUALITY
#endif
}

bool SubstrateIsTopLayerMaterial(SUBSTRATE_TOP_LAYER_TYPE In)
{
#if SUBSTRATE_NORMAL_QUALITY==0
	return In != 0;
#elif SUBSTRATE_NORMAL_QUALITY==1
	return In.y != 0;
#else
#error Unkown SUBSTRATE_NORMAL_QUALITY
#endif
}

FSubstrateTopLayerData SubstrateUnpackTopLayerData(SUBSTRATE_TOP_LAYER_TYPE In)
{
	FSubstrateTopLayerData Out = (FSubstrateTopLayerData)0;
#if SUBSTRATE_NORMAL_QUALITY==0
	Out.Material = (In & 0x3);
	Out.WorldNormal = SubstrateUnpackNormal22(In >> 10);
	Out.Roughness = UnpackR8(In >> 2);
#elif SUBSTRATE_NORMAL_QUALITY==1
	Out.Material = (In.y & 0x3);
	Out.WorldNormal = SubstrateUnpackNormal(In.x);
	Out.Roughness = UnpackR8(In.y >> 2);
#else
#error Unkown SUBSTRATE_NORMAL_QUALITY
#endif
	return Out;
}

bool IsSubstrateMaterial(FSubstrateTopLayerData In)
{
	return In.Material > TOP_LAYER_MATERIAL_NONE;
}

bool IsSingleLayerWaterMaterial(FSubstrateTopLayerData In)
{
	return In.Material == TOP_LAYER_MATERIAL_SLWATER;
}

///////////////////////////////////////////////////////////////////////////////
// Opaque rough refractions

struct FSubstrateOpaqueRoughRefractionData
{
	float	OpaqueRoughRefractionEnabled;
	float	Coverage;
	float	VarianceCm;
};

float3 SubstratePackOpaqueRoughRefractionData(FSubstrateOpaqueRoughRefractionData In)
{
	return float3(In.Coverage, In.VarianceCm, In.OpaqueRoughRefractionEnabled);
}

FSubstrateOpaqueRoughRefractionData SubstrateUnpackOpaqueRoughRefractionData(float3 In)
{
	FSubstrateOpaqueRoughRefractionData Out = (FSubstrateOpaqueRoughRefractionData)0;
	Out.Coverage = In.r;
	Out.VarianceCm = In.g;
	Out.OpaqueRoughRefractionEnabled = In.b;
	return Out;
}

// Returns the variance of refracted rays intersection with on the XY plane after scattering at the interface of a slab of matter,
// having the given Roughness and a normal along Z; after traveling along -Z from air to water (eta = 1.333).
// That variance has been generated from unit vectors, so it is a measure of the spread of light on the XY plane after a depth of 1.
// Using Thales, one can scale that spread to any distance. The Gaussian kernel can be evaluated for a depth d using a Variance = GetVarianceFromRoughness(Roughness) * d.
// 
//			| V = -Z
//			|
//			|
//			|
//			|
//			v
//	------------------- ^	Surface of roughness R
//		   /|\			|
//		  / | \			| Depth D
//		 /  |  \		|
//		|<----->|		v
//			Variance = GetVarianceFromRoughness(R) * D;
float GetVarianceFromRoughness(float Roughness)
{
	float Roughness2 = Roughness * Roughness;
	float Roughness3 = Roughness2 * Roughness;
	float Roughness4 = Roughness2 * Roughness2;
//	return saturate(0.0000226575 - 0.000332564 * Roughness - 0.0168059 * Roughness2 + 0.115569 * Roughness3 - 0.0526336 * Roughness4);	// Original
	return saturate(             - 0.000332564 * Roughness - 0.0168059 * Roughness2 + 0.115569 * Roughness3 - 0.0526336 * Roughness4);	// Removing the 0.0000226575 to get GetVarianceFromRoughness(0)=0, to not get any blurred refraction when roughness==0.
}

///////////////////////////////////////////////////////////////////////////////
// PackSSSProfile

uint PackSSSProfile(float InProfileId, float InRadius, half InThickness)
{
	uint Out = 0; 	
	Out |= (0xFF & uint(InProfileId));
	Out |= (PackR8(InRadius)<<8);
	Out |= (Pack10F(InThickness)<<16);
	return Out;
}

void UnpackSSSProfile(uint In, inout float OutProfileId, inout float OutRadius, inout half OutThickness)
{
	OutProfileId = float(BitFieldExtractU32(In, 8, 0));
	OutRadius = UnpackG8(In);
	OutThickness = Unpack10F((In >> 16) & 0x3FF);
}

///////////////////////////////////////////////////////////////////////////////
// Specular profile

uint PackSpecularProfile(float InProfileId)
{
	return asuint(InProfileId);
}

void UnpackSpecularProfile(uint In, inout float OutProfileId)
{
	OutProfileId = asfloat(In);
}

float3 EvaluateSpecularProfile(float InSpecularProfileId, float NoV, float NoL, float VoH, float NoH)
{
	const float3 ProfileCoord = GetSpecularProfileCoord(InSpecularProfileId, NoV, NoL, VoH, NoH);
	return View.SpecularProfileTexture.SampleLevel(View.SpecularProfileSampler, ProfileCoord, 0).xyz;
}

float3 EvaluateSpecularProfileEnv(float InSpecularProfileId, float NoV, float InRoughness)
{
	const float3 ProfileCoord = GetSpecularProfileEnvCoord(InSpecularProfileId, NoV, InRoughness);
	return View.SpecularProfileTexture.SampleLevel(View.SpecularProfileSampler, ProfileCoord, 0).xyz;
}

///////////////////////////////////////////////////////////////////////////////
// Glint

uint2 PackGlints(float InValue, float2 InUV)
{
	const uint Packed16 = f32tof16(InValue);
	const uint Packed8L = (Packed16 & 0xFF) << 24;
	const uint Packed8H = (Packed16 >> 8  ) << 24;

	uint2 Out;
	Out.x = PackR24F(InUV.x) | Packed8L;
	Out.y = PackR24F(InUV.y) | Packed8H;
	return Out;
}

void UnpackGlints(in uint2 InPacked, inout float OutValue, inout float2 OutUV)
{
	OutUV.x	= UnpackR24F(InPacked.x);
	OutUV.y	= UnpackR24F(InPacked.y);
	OutValue= f16tof32((InPacked.x>>24) | ((InPacked.y>>16) & 0xFF00)); 
}

///////////////////////////////////////////////////////////////////////////////
// Fuzz

uint PackFuzz(half3 InColor, half InAmount, half InRoughness, half InDither)
{
	return PackR7G7B6Gamma2(InColor, InDither) | (PackR6(InAmount, InDither) << 20) | (PackR6(InRoughness, InDither) << 26);
}

void UnpackFuzz(uint In, inout half3 OutColor, inout half OutAmount, inout half OutRoughness)
{
	OutColor     = UnpackR7G7B6Gamma2(In & 0xFFFFF);
	OutAmount    = UnpackR6((In >> 20) & 0x3F);
	OutRoughness = UnpackR6((In >> 26) & 0x3F);
}

///////////////////////////////////////////////////////////////////////////////
// Misc

// Return true if a BSDF has a custom area light integrator
bool SubstrateHasAreaLightIntegrator(in FSubstrateBSDF BSDF)
{
	const uint BSDFType = BSDF_GETTYPE(BSDF);
	return BSDFType == SUBSTRATE_BSDF_TYPE_SLAB;
}

bool SubstrateIsBSDFVisible(in FSubstrateBSDF BSDF)
{
	return any(BSDF.LuminanceWeightV > 0.0f);
}

bool SubstrateHasGreyScaleWeight(in float3 Weight)
{
	return Weight.x == Weight.y && Weight.y == Weight.z;
}

void SubstrateComputeHazeRoughnessAndWeight(
	float Roughness0,
	float Haziness,
	inout float Roughness1,
	inout float MixLobe)
{	
	// Fade out Haziness as the roughness reaches 0.5f, as otherwise it create
	//       27 |\
	// Extent   | \
	//          |  \
	//        3 |    ----
	//          0  0.25  1
	//           Rougness

	// Lerp factor for adapting haziness parameters
	const float S2 = saturate(2 * Roughness0);
	const float S4 = saturate(4 * Roughness0);

	// For low roughness, the haze extent is larger, and the core highlight is lower. 
	// This allows to emphase the haze effect at low roughness, and fade out haze as roughness increase.
	const float HazeExtent = lerp(27.0f, 3.0f, S4);
	const float HazeCore   = lerp(0.01f, 0.5f, S2);

	// Reference "A Composite BRDF Model for Hazy Gloss"
	 
	// Compute wide roughness
	float alpha_n = Square(Roughness0); 
	float lambda_h = HazeExtent;
	const float alpha_w = alpha_n * (1 + lambda_h); // Eq.8
	Roughness1 = sqrtFast(alpha_w);

	// Compute mix weights
	const float beta_h = Haziness;
	const float r_c = HazeCore;
	const float w = alpha_w;

	const float p = 1.0 / Square(1 + lambda_h); // Table 2, GGX
	const float b = 2 * (r_c * (1 - w) + w * p);
	const float D = Square(b) - 4 * (b - 1) * r_c;
	const float u = (b - sqrtFast(D)) / (2 * b - 2); // Eq. 12, 13, 14
	const float u2 = u * u;
	const float w2 = 2 * w;
	const float k_h = ((u - u2) * w2 * beta_h) / (Square(1 - u) + (u - u2) * w2 + u2); // Eq. 15
	const float r = r_c + (1 - p) * k_h; // Eq. 5
	MixLobe = k_h / r; // Eq. 4
}

float SubstrateComputeHazeRoughness(float Roughness0)				{ float Roughness1=0; float MixLobe=0; SubstrateComputeHazeRoughnessAndWeight(Roughness0, 1.0f,     Roughness1, MixLobe);	return Roughness1; }
float SubstrateComputeHazeWeight(float Roughness0, float Haziness)	{ float Roughness1=0; float MixLobe=0; SubstrateComputeHazeRoughnessAndWeight(Roughness0, Haziness, Roughness1, MixLobe);	return MixLobe; }

///////////////////////////////////////////////////////////////////////////////////////////////////
// Closure index

float SubstratePackClosureIndex(uint In)
{
	return (In + 0.5f) / 255.f;
}

uint SubstrateUnpackClosureIndex(float In)
{
	return In * 0xFF;
}

uint4 SubstrateUnpackClosureIndex(float4 In)
{
	return In * 0xFF;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FSubstrateData
{
	FSubstrateBSDF InlinedBSDF;// Used for parameter blending or special BSDF such as hair, cloud or water.

	int  OperatorIndex;		// Used as entry point ot the material graph with multiple BSDF and operators.

	#if USE_DEVELOPMENT_SHADERS
	float3 PreviewColor;	// Used as preview in the material graph within the editor.
	#endif
};

FSubstrateData GetInitialisedSubstrateData()
{
	FSubstrateData SubstrateData = (FSubstrateData)0;
	return SubstrateData;
}

///////////////////////////////////////////////////////////////////////////////
// Material & BSDF description

struct FSubstrateOperator
{
	// SUBSTRATE_TODO: pack in a single uint
	int ParentIndex;
	uint LayerDepth;
	uint MaxDistanceFromLeaves;							// The largest depth distance from any tree leaf representing a BSDF
	uint Type;											// The type of operator
	int	LeftIndex;
	int RightIndex;
	float Weight;

	// Coverage / Transmittance

	float  Coverage;									// The summarised coverage of all the matter (from the sub tree) represented by that operator
	float3 ThroughputAlongV;							// This is the throughput of the matter represented by this operator (matter and fuzz transmittance). If coverage is 0.25, then 25% of the light will be affected by the transmittance and 75% of light won't be.
	float3 TransmittanceAlongN;							// This is the transmittance of the matter represented by this operator. If coverage is 0.25, then 25% of the light will be affected by the transmittance and 75% of light won't be.

	float  VerticalTop_Coverage;						// Only used by vertical layering operators. It represent the coverage of all the matter of the top layer (from the left sub tree).
	float3 VerticalTop_ThroughputAlongV;				// Only used by vertical layering operators. It represent the transmittance of matter of the top layer (from the left sub tree, and towards the view).
	float3 VerticalTop_TransmittanceAlongN;				// Only used by vertical layering operators. It represent the transmittance of matter of the top layer (from the left sub tree, and along the normal of the surface).

	float  VerticalBot_Coverage;						// Analoguous to VerticalTop_ data but for the bottom layer.
	float3 VerticalBot_ThroughputAlongV;				// Idem.
	float3 VerticalBot_TransmittanceAlongN;				// Idem.

	// Rough refraction
	float3				 TopDownRefractionWorldNormal;	// The normal used for the single refracted direction
	FSubstrateLobeStatistic TopDownRefractionLobeStat;		// The lobe used for the scene rough refraction from front to back, e.g. used for translucent rough refraciton
	FSubstrateLobeStatistic BottomUpRefractionLobeStat;	// The lobe used for the scene rough refraction from back to front, e.g. used for opaque rough refraction of incoming light within sub layers

	// Opaque rough refraction
	float				 OpaqueRoughRefractCoverage;	// The coverage of the top layer to account for in order to fade out the influence on the blur.
	float				 OpaqueRoughRefractThicknessCm;	// The thickness of stack material from the top layer to the bottom layer.
	float				 OpaqueRoughRefractTopRoughness;// The roughness of the top layer. This is enough because we currently assume all the bottom layers have a constant IOR, so eta will after that always be 1 and no refraction will occur after.

	// Translucent refractions
	FSubstrateLobeStatistic VerticalTop_TopDownRefractionLobeStat;	// Only used by vertical layering operators. It represent the lobe statistics of the top layer at its interface.
	FSubstrateLobeStatistic VerticalTop_BottomUpRefractionLobeStat;
};

FSubstrateOperator GetInitialisedSubstrateOperator()
{
	FSubstrateOperator SubstrateOp = (FSubstrateOperator)0;
	SubstrateOp.ParentIndex = -1;
	SubstrateOp.LeftIndex = -1;
	SubstrateOp.RightIndex = -1;
	return SubstrateOp;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
struct FSubstratePixelHeader;
struct FSubstrateIntegrationSettings;
#endif

// Representes a Substrate material as BSDFs organised in a tree repreesnting its topology 
struct FSubstrateTree
{
	int BSDFCount;
	uint OperatorCount;

	#if ALLOW_SUBSTRATE_TREE
	FSubstrateBSDF BSDFs[SUBSTRATE_MAX_CLOSURE_COUNT];
	FSubstrateOperator Operators[SUBSTRATE_MAX_OPERATOR_COUNT];
	#else
	FSubstrateBSDF BSDFs[1];
	FSubstrateOperator Operators[1];
	#endif

	// Certain compiler don't support struct forward declaration. This function depends on FSubstratePixelHeader, 
	// which depends on FSubstrateTree. To overcome circular dependency without forward declaration, a standalone 
	// version of this function exists as well.
	#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
	void UpdateSingleBSDFOperatorCoverageTransmittance(
		FSubstratePixelHeader SubstratePixelHeader,
		int BSDFIndex,
		FSubstrateIntegrationSettings Settings,
		float3 V);
	#endif

	void UpdateSingleOperatorCoverageTransmittance(
		int OpIndex);

	void UpdateAllBSDFWithBottomUpOperatorVisit_Weight(
		int BSDFIndex,
		int OpIndex,
		int PreviousIsInputA);

	void UpdateAllBSDFWithBottomUpOperatorVisit_Horizontal(
		int BSDFIndex,
		int OpIndex,
		int PreviousIsInputA);

	void UpdateAllBSDFWithBottomUpOperatorVisit_Vertical(
		int BSDFIndex,
		int OpIndex,
		int PreviousIsInputA);

	FSubstrateData SubstrateAdd(FSubstrateData A, FSubstrateData B, int OperatorIndex, uint MaxDistanceFromLeaves);
	FSubstrateData SubstrateWeight(FSubstrateData A, float Weight, int OperatorIndex, uint MaxDistanceFromLeaves);
	FSubstrateData SubstrateHorizontalMixing(FSubstrateData Background, FSubstrateData Foreground, float Mix, int OperatorIndex, uint MaxDistanceFromLeaves);
	FSubstrateData SubstrateVerticalLayering(FSubstrateData Top, FSubstrateData Base, int OperatorIndex, uint MaxDistanceFromLeaves);
	FSubstrateData PromoteParameterBlendedBSDFToOperator(FSubstrateData SubstrateData, int OperatorIndex, int BSDFIndex, int LayerDepth, int bIsBottom);
};

FSubstrateTree GetInitialisedSubstrateTree()
{
#if !COMPILER_FXC
	FSubstrateTree Out;
	Out.BSDFCount = 0;
	Out.OperatorCount = 0;
	// Also set first BSDF and operator to make sure unlit still works and all data is isnitialised with empty graph.
	Out.BSDFs[0] = (FSubstrateBSDF)0;
	Out.Operators[0] = (FSubstrateOperator)0;
	return Out;
#else
	return (FSubstrateTree)0;
#endif
}

void SubstratePackOutSubstrateTree(
	RWTexture2DArray<uint> ExtraMaterialDataUAV,
	in uint RootOperatorIndex,
	in FSubstrateTree SubstrateTree,
	in int SliceStoringDebugSubstrateTreeData)
{
#define WRITE_INT(Data) ExtraMaterialDataUAV[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)] = asuint(Data);
#define WRITE_UINT(Data) ExtraMaterialDataUAV[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)] = Data;
#define WRITE_FLOAT(Data) ExtraMaterialDataUAV[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)] = asuint(Data);
#define WRITE_FLOAT3(Data) WRITE_FLOAT(Data.x);WRITE_FLOAT(Data.y);WRITE_FLOAT(Data.z);
#define WRITE_FLOAT4(Data) WRITE_FLOAT(Data.x);WRITE_FLOAT(Data.y);WRITE_FLOAT(Data.z);WRITE_FLOAT(Data.w);

	// Write out header
	int OutCoord = 0;
	int OutCoordY = 0;

	WRITE_UINT(SUBSTRATE_USES_CONVERSION_FROM_LEGACY ? 1 : 0);
	WRITE_UINT(SubstrateTree.BSDFCount);
	WRITE_UINT(SubstrateTree.OperatorCount);
	WRITE_UINT(RootOperatorIndex);

	// Write out BSDFs
	OutCoord = 0;
	OutCoordY = 1;

	SubstrateTree.BSDFCount = min(SubstrateTree.BSDFCount, SUBSTRATE_CLAMPED_CLOSURE_COUNT);
	Substrate_for_unroll(int i=0, i< SubstrateTree.BSDFCount, ++i)
	{
		WRITE_UINT(SubstrateTree.BSDFs[i].State);
		WRITE_INT(SubstrateTree.BSDFs[i].OperatorIndex);
		WRITE_FLOAT3(SubstrateTree.BSDFs[i].LuminanceWeightV);
		WRITE_FLOAT(SubstrateTree.BSDFs[i].CoverageAboveAlongN);
		WRITE_FLOAT3(SubstrateTree.BSDFs[i].TransmittanceAboveAlongN);
		WRITE_UINT(SubstrateTree.BSDFs[i].bIsBottom);
		WRITE_UINT(SubstrateTree.BSDFs[i].bIsTop);

	#if SUBSTRATE_INLINE_SHADING
		WRITE_FLOAT(SubstrateTree.BSDFs[i].Coverage);

		WRITE_FLOAT3(SubstrateTree.BSDFs[i].Emissive);
		WRITE_FLOAT(SubstrateTree.BSDFs[i].ThicknessCm);
		WRITE_FLOAT3(SubstrateTree.BSDFs[i].TmpMFP);

		WRITE_FLOAT(SubstrateTree.BSDFs[i].TopLayerDataWeight);
	#endif

		// Skipping specific single layer water data

		WRITE_FLOAT4(SubstrateTree.BSDFs[i].VGPRs[0]);
		WRITE_FLOAT4(SubstrateTree.BSDFs[i].VGPRs[1]);
		WRITE_FLOAT4(SubstrateTree.BSDFs[i].VGPRs[2]);
		WRITE_FLOAT4(SubstrateTree.BSDFs[i].VGPRs[3]);
		WRITE_FLOAT4(SubstrateTree.BSDFs[i].VGPRs[4]);
	}

	// Write out operators
	OutCoord = 0;
	OutCoordY = 2;

	SubstrateTree.OperatorCount = min(SubstrateTree.OperatorCount, SUBSTRATE_MAX_OPERATOR_COUNT);
	SUBSTRATE_UNROLL_N(SUBSTRATE_MAX_OPERATOR_COUNT)
	for (uint j = 0; j < SubstrateTree.OperatorCount; ++j)
	{
		WRITE_INT(SubstrateTree.Operators[j].ParentIndex);
		WRITE_UINT(SubstrateTree.Operators[j].LayerDepth);
		WRITE_UINT(SubstrateTree.Operators[j].MaxDistanceFromLeaves);
		WRITE_UINT(SubstrateTree.Operators[j].Type);
		WRITE_INT(SubstrateTree.Operators[j].LeftIndex);
		WRITE_INT(SubstrateTree.Operators[j].RightIndex);
		WRITE_FLOAT(SubstrateTree.Operators[j].Weight);
		WRITE_FLOAT(SubstrateTree.Operators[j].Coverage);
		// Skip the rest for now
	}

#undef WRITE_UINT
#undef WRITE_FLOAT
#undef WRITE_FLOAT3
#undef WRITE_FLOAT4
}

struct FSubstrateTreeHeader
{
	int ConvertedFromLegacy;
	int BSDFCount;
	int OperatorCount;
	uint RootOperatorIndex;
};

#define READ_INT(Data) Data = asint(SubstrateMaterialData[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)]);
#define READ_UINT(Data) Data = SubstrateMaterialData[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)];
#define READ_FLOAT(Data) Data = asfloat(SubstrateMaterialData[uint3(OutCoord++, OutCoordY, SliceStoringDebugSubstrateTreeData)]);
#define READ_FLOAT3(Data) READ_FLOAT(Data.x);READ_FLOAT(Data.y);READ_FLOAT(Data.z);
#define READ_FLOAT4(Data) READ_FLOAT(Data.x);READ_FLOAT(Data.y);READ_FLOAT(Data.z);READ_FLOAT(Data.w);

void SubstrateUnpackInSubstrateTreeHeader(
	Texture2DArray<uint> SubstrateMaterialData,
	inout FSubstrateTreeHeader SubstrateTreeHeader,
	in int SliceStoringDebugSubstrateTreeData)
{
	SubstrateTreeHeader = (FSubstrateTreeHeader)0;

	int OutCoord = 0;
	int OutCoordY= 0;

	READ_UINT(SubstrateTreeHeader.ConvertedFromLegacy);
	READ_UINT(SubstrateTreeHeader.BSDFCount);
	READ_UINT(SubstrateTreeHeader.OperatorCount);
	READ_UINT(SubstrateTreeHeader.RootOperatorIndex);
}

FSubstrateBSDF SubstrateUnpackInSubstrateTreeBSDF(
	uint BSDFIndex,
	Texture2DArray<uint> SubstrateMaterialData,
	inout FSubstrateTreeHeader SubstrateTreeHeader,
	in int SliceStoringDebugSubstrateTreeData)
{
	int OutCoord = BSDFIndex * (SUBSTRATE_INLINE_SHADING ? 40 : 31);		// Skip BSDFs
	int OutCoordY= 1;

	FSubstrateBSDF BSDF = (FSubstrateBSDF)0;

	READ_UINT(BSDF.State);
	READ_INT(BSDF.OperatorIndex);
	READ_FLOAT3(BSDF.LuminanceWeightV);
	READ_FLOAT(BSDF.CoverageAboveAlongN);
	READ_FLOAT3(BSDF.TransmittanceAboveAlongN);
	READ_UINT(BSDF.bIsBottom);
	READ_UINT(BSDF.bIsTop);

#if SUBSTRATE_INLINE_SHADING
	READ_FLOAT(BSDF.Coverage);

	READ_FLOAT3(BSDF.Emissive);
	READ_FLOAT(BSDF.ThicknessCm);
	READ_FLOAT3(BSDF.TmpMFP);

	READ_FLOAT(BSDF.TopLayerDataWeight);
#endif

	// Skipping specific single layer water data

	READ_FLOAT4(BSDF.VGPRs[0]);
	READ_FLOAT4(BSDF.VGPRs[1]);
	READ_FLOAT4(BSDF.VGPRs[2]);
	READ_FLOAT4(BSDF.VGPRs[3]);
	READ_FLOAT4(BSDF.VGPRs[4]);

	return BSDF;
}

FSubstrateOperator SubstrateUnpackInSubstrateTreeOperator(
	uint OperatorIndex,
	Texture2DArray<uint> SubstrateMaterialData,
	inout FSubstrateTreeHeader SubstrateTreeHeader,
	in int SliceStoringDebugSubstrateTreeData)
{
	int OutCoord = OperatorIndex * 8;	// Skip Operators
	int OutCoordY = 2;

	FSubstrateOperator Op = (FSubstrateOperator)0;

	READ_INT(Op.ParentIndex);
	READ_UINT(Op.LayerDepth);
	READ_UINT(Op.MaxDistanceFromLeaves);
	READ_UINT(Op.Type);
	READ_INT(Op.LeftIndex);
	READ_INT(Op.RightIndex);
	READ_FLOAT(Op.Weight);
	READ_FLOAT(Op.Coverage);

	return Op;
}

#undef READ_INT
#undef READ_UINT
#undef READ_FLOAT
#undef READ_FLOAT3
#undef READ_FLOAT4

struct FSubstratePixelFootprint
{
	float PixelRadiusInWorldSpace;	// In cm
	float NormalCurvatureRoughness; // Surface curvature converted into roughness for AA purpose
};

FSubstratePixelFootprint SubstrateGetPixelFootprint(float3 dPdx, float3 dPdy, float InNormalCurvatureRoughness)
{
	const float dX = lengthFast(dPdx);
	const float dY = lengthFast(dPdy);

	FSubstratePixelFootprint Out;
	Out.PixelRadiusInWorldSpace = min(dX, dY) * 0.5f;
	Out.NormalCurvatureRoughness = InNormalCurvatureRoughness;
	return Out;
}

FSubstratePixelFootprint SubstrateGetPixelFootprint(float3 InWorldPosition, float InNormalCurvatureRoughness)
{
#if CLOUD_LAYER_PIXEL_SHADER==1
	// Clouds materials feature dynamic loops and thus cannot support derivatives.
	return (FSubstratePixelFootprint)0;
#else
	const float3 dPdx = ddx(InWorldPosition);
	const float3 dPdy = ddy(InWorldPosition);
	return SubstrateGetPixelFootprint(dPdx, dPdy, InNormalCurvatureRoughness);
#endif
}

///////////////////////////////////////////////////////////////////////////////
// BSDF
#if USE_ANALYTIC_DERIVATIVES
#define GLINT_UV_TYPE FloatDeriv2
#else
#define GLINT_UV_TYPE float2
#endif

FSubstrateData InternalGetSubstrateSlabBSDF(
	FSubstratePixelFootprint InPixelFootprint,
	float3 RawNormal,
	float3 DiffuseAlbedo, float3 F0, float3 F90,
	float Roughness, float Anisotropy,
	uint SSSProfileID, bool bSupportDefaultSSSProfile, float3 SSSMFP, float SSSMFPScale, float SSSPhaseAnisotropy, uint SSSType,
	float3 Emissive, 
	float SecondRoughness, float SecondRoughnessWeight, float SecondRoughnessAsSimpleClearCoat,
	float ClearCoatUseSecondNormal, float3 ClearCoatBottomNormal,
	float FuzzAmount, float3 FuzzColor, float FuzzRoughness,
	float GlintValue, float2 GlintUV, float2 GlintUVdx, float2 GlintUVdy,
	float SpecularProfileId,
	float ThicknessCm,
	bool bIsThinSurface,
	bool bIsAtTheBottomOfTopology,	// Make sure we do not affect MFP when wrap mode is selected. This is also an optimisation, disabling lots of code when we know about topology.
	uint SharedLocalBasisIndex, inout uint SharedLocalBasisTypes,
	bool bSupportSSSDiffusion)
{
	// Curvature-based roughness for geometric AA
	Roughness		= max(Roughness, InPixelFootprint.NormalCurvatureRoughness);
	SecondRoughness = max(SecondRoughness, InPixelFootprint.NormalCurvatureRoughness);
	FuzzRoughness	= max(FuzzRoughness, InPixelFootprint.NormalCurvatureRoughness);

	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

	SSSMFPScale = saturate(SSSMFPScale);
	const bool bSSSProfilePlugged = SSSProfileID >= (bSupportDefaultSSSProfile ? 0u : 1u);
	const bool bMFPPlugged = any(SSSMFP > 0);
	const bool bIsThin = bIsAtTheBottomOfTopology && (bMFPPlugged && bIsThinSurface); // Enable thin only if MFP is plugged, as otherwise there will be no back-tramission

	// In forward mode (blend mode != Opaque/Masked), diffusion is not supported. 
	// In such a case we fallback onto no SSS (i.e., diffuse), as look-wise, this is the closest one.
	if (!bSupportSSSDiffusion && (SSSType == SSS_TYPE_DIFFUSION || SSSType == SSS_TYPE_DIFFUSION_PROFILE))
	{
		SSSType = SSS_TYPE_NONE;
	}

	if (!bIsAtTheBottomOfTopology)
	{
		SSSType = SSS_TYPE_SIMPLEVOLUME;
	}
	else if (SSSType == SSS_TYPE_DIFFUSION && bSSSProfilePlugged && SSSMFPScale > 0.f)
	{
		SSSType = SSS_TYPE_DIFFUSION_PROFILE;
	}
	else if (!bMFPPlugged)
	{
		// If MFP is 0, then skip SSS, unless this is a Diffusion profile which has MFP in the profile itself.
		SSSType = SSS_TYPE_NONE;
	}

	FHaziness Haziness = InitialiseHaziness();

	// Derive roughness and haziness from profile data
	if (!SUBSTRATE_FASTPATH && SSSType == SSS_TYPE_DIFFUSION_PROFILE)
	{
	#if SUBSTRATE_GBUFFER_FORMAT == 1
		// Adaptive GBuffer: Evaluate dual specular roughnesses and store as roughness & haziness parameters.
		GetSubsurfaceProfileDualSpecular(SSSProfileID, Roughness, SSSMFPScale, Roughness, Haziness.Roughness, Haziness.Weight);
	#else
		// For the legacy gbuffer path, we load the GBuffer and convert it to a BSDF by calling InternalGetSubstrateSlabBSDF from SubstrateReadGBufferBSDF.
		// But we cannot load any haziness from that limited GBuffer. We still need to recover the haziness parameters for dual specular used with Subsurface Profile shading model.
		// We do this by using the SecondRoughness input parameters.
		Haziness.Roughness = SecondRoughness;
		Haziness.Weight = SecondRoughnessWeight;
		Haziness.bSimpleClearCoat = SecondRoughnessAsSimpleClearCoat > 0.0f;
	#endif

		SSSMFP = GetSubsurfaceProfileMFPInCm(SSSProfileID).xyz * SSSMFPScale;
	}
	else
	{
		Haziness.Roughness = SecondRoughness;
		Haziness.Weight = SecondRoughnessWeight;
		Haziness.bSimpleClearCoat = SecondRoughnessAsSimpleClearCoat > 0.0f;

	#if CLEAR_COAT_BOTTOM_NORMAL
		if (ClearCoatUseSecondNormal > 0.0f)
		{
			float2 oct2 = UnitVectorToOctahedron(normalize(RawNormal));
			float2 oct1 = UnitVectorToOctahedron(ClearCoatBottomNormal);
			Haziness.BottomNormalOct = ((oct1 - oct2) * 0.25) + (128.0 / 255.0);
		}
		else
	#endif
		{
			Haziness.BottomNormalOct = DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT;
		}
	}

	// Convert MFP be expressed relative to simple volume thickness
	// Note: MFP rescaling for SSS_TYPE_SIMPLEVOLUME is done later, as the thickness needs to be preserved (i.e. not normalized) for rough refraction computation
	if (!SUBSTRATE_FASTPATH && (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP))
	{
		SSSMFP = RescaleMFPToComputationSpace(SSSMFP, ThicknessCm, SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM);
		ThicknessCm = SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM;
	}

	// When Glints are enabled, we need to not keep haziness even R0=R1 to avoid pops (second lobe disapearing, more energy from glints).
	const bool bHasGlint = SUBSTRATE_COMPLEXSPECIALPATH && GlintValue < 1.0f;
	const uint bHasHaziness = (((Haziness.Roughness != Roughness || bHasGlint) && Haziness.Weight > 0.0f) || Haziness.bSimpleClearCoat) ? 1 : 0;
	const uint bHasFuzz = FuzzAmount > 0.0f ? 1 : 0;
#if MATERIAL_USES_ANISOTROPY
	const uint bHasAnisotropy = Anisotropy != 0 ? 1 : 0;
#else
	Anisotropy = 0.0f;
	const uint bHasAnisotropy = 0;
#endif
	if (!SUBSTRATE_FASTPATH && bHasAnisotropy)
	{
		SubstrateRequestSharedLocalBasisTangent(SharedLocalBasisTypes, SharedLocalBasisIndex);
	}

	// About the thin, a.k.a. two sided lighting, model:
	// 	- Thin lighting model is used when MFP is plugged and the surface is flagged as 'is-thin-surface'
	//  - It cannot work with a SSS Profile because the profile id is stored in the register used for the MFP (e.g. see SLAB_SSSPROFILEID).
	// 	- Thin lighting model force disabled the SSSProfile.
	//  - It can work with the Substrate SSS because the MFP is shared in this case between both.
	//  - We also only enabled thin lighting is the MFP input pin is plugged in.
	//  - It will only be ran for the bottom layer only. Otherwise, the simple volume should be used. As such, Two-sided lighting can also be disabled later if part of the bottom layer.

	BSDF_SETTYPE			(SubstrateData.InlinedBSDF,		SUBSTRATE_BSDF_TYPE_SLAB);
	BSDF_SETSHAREDLOCALBASISID(SubstrateData.InlinedBSDF,	SharedLocalBasisIndex);
	BSDF_SETEMISSIVE		(SubstrateData.InlinedBSDF,		Emissive);

#if (SUBSTRATE_FASTPATH==0 || MATERIALBLENDING_ANY_TRANSLUCENT) // !Fast path or rendering translucent materials
	BSDF_SETSSSTYPE			(SubstrateData.InlinedBSDF,		SSSType);
	BSDF_SETHASANISOTROPY	(SubstrateData.InlinedBSDF,		bHasAnisotropy);
	BSDF_SETISTOPLAYER		(SubstrateData.InlinedBSDF,		0);
	BSDF_SETISTHIN			(SubstrateData.InlinedBSDF,		bIsThin ? 1 : 0);
	BSDF_SETHASMFP			(SubstrateData.InlinedBSDF,		bMFPPlugged ? 1 : 0);
	BSDF_SETHASHAZINESS		(SubstrateData.InlinedBSDF,		bHasHaziness);
	BSDF_SETHASFUZZ			(SubstrateData.InlinedBSDF,		bHasFuzz);
	BSDF_SETTHICKNESSCM		(SubstrateData.InlinedBSDF,		ThicknessCm);
#endif

#if SUBSTRATE_GLINTS_ENABLED
	if (bHasGlint)
	{
		BSDF_SETHASGLINT(SubstrateData.InlinedBSDF, 1);
		SLAB_GLINT_VALUE(SubstrateData.InlinedBSDF)		= GlintValue;
		SLAB_GLINT_UV(SubstrateData.InlinedBSDF)		= GlintUV;
		SLAB_GLINT_UVDDX(SubstrateData.InlinedBSDF)		= GlintUVdx;
		SLAB_GLINT_UVDDY(SubstrateData.InlinedBSDF)		= GlintUVdy;
	}
#endif // SUBSTRATE_GLINTS_ENABLED
#if SUBSTRATE_SPECPROFILE_ENABLED
	if (SpecularProfileId != 0.0f)
	{
		BSDF_SETHASSPECPROFILE	(SubstrateData.InlinedBSDF, 1);
		SLAB_SPECPROFILEID		(SubstrateData.InlinedBSDF) = SpecularProfileId * 255; // Stored as 0-255 index, and sign store parametrization type (View/Light or Half-Angle)
	}
#endif // SUBSTRATE_SPECPROFILE_ENABLED
	
#if !SUBSTRATE_FASTPATH
	// See how UMaterialExpressionSubstrateSlabBSDF node interface is changed
	BSDF_SETHASF90(SubstrateData.InlinedBSDF, any(F90 < 1.0f));
#endif

	SLAB_DIFFUSEALBEDO(SubstrateData.InlinedBSDF)			= DiffuseAlbedo;
	SLAB_F0(SubstrateData.InlinedBSDF)						= F0;
	SLAB_F90(SubstrateData.InlinedBSDF)						= F90;

	SLAB_ROUGHNESS			(SubstrateData.InlinedBSDF)		= Roughness;

#if !SUBSTRATE_FASTPATH
	SLAB_ANISOTROPY			(SubstrateData.InlinedBSDF)		= Anisotropy;

	SLAB_HAZINESS(SubstrateData.InlinedBSDF)					= PackHaziness(Haziness);

	if (SSSType == SSS_TYPE_DIFFUSION_PROFILE)
	{
		SLAB_SSSPROFILEID	(SubstrateData.InlinedBSDF)			= float(SSSProfileID);
		SLAB_SSSPROFILERADIUSSCALE(SubstrateData.InlinedBSDF)	= SSSMFPScale;
		SLAB_SSSPROFILETHICKNESSCM(SubstrateData.InlinedBSDF)	= ThicknessCm;
	}
	else
	{
		SLAB_SSSMFP	(SubstrateData.InlinedBSDF)				= SSSMFP * SSSMFPScale;
	}

	SLAB_SSSPHASEANISOTROPY(SubstrateData.InlinedBSDF)			= SSSPhaseAnisotropy;

	SLAB_FUZZ_AMOUNT		(SubstrateData.InlinedBSDF)		= FuzzAmount;
	SLAB_FUZZ_COLOR			(SubstrateData.InlinedBSDF)		= FuzzColor;
	SLAB_FUZZ_ROUGHNESS		(SubstrateData.InlinedBSDF)		= FuzzRoughness;

#endif // !SUBSTRATE_FASTPATH

#if SUBSTRATE_INLINE_SHADING
	// At this stage we do not know if this Substrate BSDF is going to be using simple volume lighting. So we always store the MFP on the side to not override SSSPROFILE radius scale sharing the same register.
	SubstrateData.InlinedBSDF.TmpMFP = SSSMFP * SSSMFPScale;

	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor = lerp(DiffuseAlbedo, F0, F0RGBToMetallic(F0));
#endif

#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL || !PLATFORM_ENABLES_SUBSTRATE_GLINTS
	if (GlintValue < 1)
	{
		// In this case, SUBSTRATE_GLINTS_ENABLED is typically 0. So no glint data will be written to the Substrate buffer.
		// So it is here that we apply glint coverage to reflectivity, for reflection to better match in Lumen reflection, ray tracing or lower platforms.
		BSDF_SETHASGLINT(SubstrateData.InlinedBSDF, 0);
		GlintValue = saturate(GlintValue);
		SLAB_F0(SubstrateData.InlinedBSDF)  *= GlintValue;
		SLAB_F90(SubstrateData.InlinedBSDF) *= GlintValue;
	}
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateSlabBSDF(
	FSubstratePixelFootprint InPixelFootprint,
	float3 RawNormal,
	float3 DiffuseAlbedo, float3 F0, float3 F90,
	float Roughness, float Anisotropy,
	uint SSSProfileID, bool bSupportDefaultSSSProfile, float3 SSSMFP, float SSSMFPScale, float SSSPhaseAnisotropy, uint SSSType,
	float3 Emissive,
	float SecondRoughness, float SecondRoughnessWeight, float SecondRoughnessAsSimpleClearCoat,
	float ClearCoatUseSecondNormal, float3 ClearCoatBottomNormal,
	float FuzzAmount, float3 FuzzColor, float FuzzRoughness,
	float GlintValue, FloatDeriv2 GlintUV,
	float SpecularProfileId,
	float ThicknessCm,
	bool bIsThinSurface,
	bool bIsAtTheBottomOfTopology,
	uint SharedLocalBasisIndex, inout uint SharedLocalBasisTypes)
{
	#if ALLOW_SUBSTRATE_TREE
	return InternalGetSubstrateSlabBSDF(
		InPixelFootprint,
		RawNormal,
		DiffuseAlbedo, F0, F90,
		Roughness, Anisotropy,
		SSSProfileID, bSupportDefaultSSSProfile, SSSMFP, SSSMFPScale, SSSPhaseAnisotropy, SSSType,
		Emissive,
		SecondRoughness, SecondRoughnessWeight, SecondRoughnessAsSimpleClearCoat,
		ClearCoatUseSecondNormal, ClearCoatBottomNormal,
		FuzzAmount, FuzzColor, FuzzRoughness,
		GlintValue, GlintUV.Value, GlintUV.Ddx, GlintUV.Ddy,
		SpecularProfileId,
		ThicknessCm,
		bIsThinSurface,
		bIsAtTheBottomOfTopology,
		SharedLocalBasisIndex, SharedLocalBasisTypes,
		SUBSTRATE_OPAQUE_MATERIAL /*bSupportSSSDiffusion*/);
	#else
	return (FSubstrateData)0;
	#endif
}


FSubstrateData GetSubstrateSlabBSDF(
	FSubstratePixelFootprint InPixelFootprint,
	float3 RawNormal,
	float3 DiffuseAlbedo, float3 F0, float3 F90,
	float Roughness, float Anisotropy,
	uint SSSProfileID, bool bSupportDefaultSSSProfile, float3 SSSMFP, float SSSMFPScale, float SSSPhaseAnisotropy, uint SSSType,
	float3 Emissive,
	float SecondRoughness, float SecondRoughnessWeight, float SecondRoughnessAsSimpleClearCoat,
	float ClearCoatUseSecondNormal, float3 ClearCoatBottomNormal,
	float FuzzAmount, float3 FuzzColor, float FuzzRoughness,
	float GlintValue, float2 GlintUV,
	float SpecularProfileId,
	float ThicknessCm,
	bool bIsThinSurface,
	bool bIsAtTheBottomOfTopology,
	uint SharedLocalBasisIndex, inout uint SharedLocalBasisTypes)
{
	#if ALLOW_SUBSTRATE_TREE
	return InternalGetSubstrateSlabBSDF(
		InPixelFootprint,
		RawNormal,
		DiffuseAlbedo, F0, F90,
		Roughness, Anisotropy,
		SSSProfileID, bSupportDefaultSSSProfile, SSSMFP, SSSMFPScale, SSSPhaseAnisotropy, SSSType,
		Emissive,
		SecondRoughness, SecondRoughnessWeight, SecondRoughnessAsSimpleClearCoat,
		ClearCoatUseSecondNormal, ClearCoatBottomNormal,
		FuzzAmount, FuzzColor, FuzzRoughness,
		GlintValue, GlintUV, ddx(GlintUV), ddy(GlintUV),
		SpecularProfileId,
		ThicknessCm,
		bIsThinSurface,
		bIsAtTheBottomOfTopology,
		SharedLocalBasisIndex, SharedLocalBasisTypes,
		SUBSTRATE_OPAQUE_MATERIAL /*bSupportSSSDiffusion*/);
	#else
	return (FSubstrateData)0;
	#endif
}

FSubstrateData GetSubstrateVolumeFogCloudBSDF(float3 Albedo, float3 Extinction, float3 Emissive, float AmbientOcclusion)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	VOLUMETRICFOGCLOUD_ALBEDO		(SubstrateData.InlinedBSDF) = Albedo;
	VOLUMETRICFOGCLOUD_EXTINCTION	(SubstrateData.InlinedBSDF) = Extinction;
	VOLUMETRICFOGCLOUD_AO			(SubstrateData.InlinedBSDF) = AmbientOcclusion;

#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= Albedo;
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateUnlitBSDF(float3 Emissive, float3 TransmittanceColor, float3 Normal)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_UNLIT);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	UNLIT_TRANSMITTANCE				(SubstrateData.InlinedBSDF) = TransmittanceColor;
	UNLIT_NORMAL					(SubstrateData.InlinedBSDF) = Normal;

#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= Emissive;
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateHairBSDF(float3 BaseColor, float Scatter, float Specular, float Roughness, float Backlit, float3 Emissive, bool bComplexTransmittance, uint SharedLocalBasisIndex)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

	// Enable complex transmittance only for hair using cards/strands vertex factories
	#ifdef USE_HAIR_COMPLEX_TRANSMITTANCE
	  #if USE_HAIR_COMPLEX_TRANSMITTANCE == 0
		bComplexTransmittance = false;
	  #endif
	#elif defined(HAIR_CARD_MESH_FACTORY) || defined(HAIR_STRAND_MESH_FACTORY)
		bComplexTransmittance = true;
	#endif

	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_HAIR);
	BSDF_SETSHAREDLOCALBASISID		(SubstrateData.InlinedBSDF, SharedLocalBasisIndex);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	HAIR_BASECOLOR					(SubstrateData.InlinedBSDF) = BaseColor;
	HAIR_SCATTER					(SubstrateData.InlinedBSDF) = Scatter;
	HAIR_ROUGHNESS					(SubstrateData.InlinedBSDF) = Roughness;
	HAIR_SPECULAR					(SubstrateData.InlinedBSDF) = Specular;
	HAIR_BACKLIT					(SubstrateData.InlinedBSDF) = Backlit;
	HAIR_COMPLEXTRANSMITTANCE		(SubstrateData.InlinedBSDF) = bComplexTransmittance;

#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= BaseColor;
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateEyeBSDF(float3 DiffuseAlbedo, float3 F0, float Roughness, float IrisMask, float IrisDistance, float3 IrisNormal, float3 IrisPlaneNormal, uint SSSProfileId, float3 Emissive, uint SharedLocalBasisIndex)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();
	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_EYE);
	BSDF_SETSHAREDLOCALBASISID		(SubstrateData.InlinedBSDF, SharedLocalBasisIndex);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	EYE_DIFFUSEALBEDO				(SubstrateData.InlinedBSDF) = DiffuseAlbedo;
	EYE_ROUGHNESS					(SubstrateData.InlinedBSDF) = Roughness;
	EYE_IRISDISTANCE				(SubstrateData.InlinedBSDF) = IrisDistance;
	EYE_IRISMASK					(SubstrateData.InlinedBSDF) = IrisMask;
	EYE_IRISNORMAL					(SubstrateData.InlinedBSDF) = IrisNormal;
	EYE_IRISPLANENORMAL				(SubstrateData.InlinedBSDF) = IrisPlaneNormal;
	EYE_SSSPROFILEID				(SubstrateData.InlinedBSDF) = float(SSSProfileId);
	EYE_F0							(SubstrateData.InlinedBSDF) = F0;
	EYE_F90							(SubstrateData.InlinedBSDF) = saturate(F0 * 50.f);

	// Legacy eye shading always enables SSS on eye material. If no SSS profile is provided, Profile=0 will be used.
	const bool bHasSSS = true;
	BSDF_SETSSSTYPE(SubstrateData.InlinedBSDF, bHasSSS ? SSS_TYPE_DIFFUSION_PROFILE : SSS_TYPE_NONE);
#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= DiffuseAlbedo;
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateEyeBSDF(float3 DiffuseAlbedo, float Roughness, float IrisMask, float IrisDistance, float3 IrisNormal, float3 IrisPlaneNormal, uint SSSProfileId, float3 Emissive, uint SharedLocalBasisIndex)
{
	return GetSubstrateEyeBSDF(DiffuseAlbedo, SUBSTRATE_EYE_DEFAULT_F0, Roughness, IrisMask, IrisDistance, IrisNormal, IrisPlaneNormal, SSSProfileId, Emissive, SharedLocalBasisIndex);
}

FSubstrateData GetSubstrateSingleLayerWaterBSDF(
	float3 BaseColor, float Metallic, float Specular, float Roughness, float3 Emissive, float TopMaterialOpacity,
	float3 WaterAlbedo, float3 WaterExtinction, float WaterPhaseG, float3 ColorScaleBehindWater, uint SharedLocalBasisIndex)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER);
	BSDF_SETSHAREDLOCALBASISID		(SubstrateData.InlinedBSDF, SharedLocalBasisIndex);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	SLW_BASECOLOR					(SubstrateData.InlinedBSDF) = BaseColor;
	SLW_METALLIC					(SubstrateData.InlinedBSDF) = Metallic;
	SLW_SPECULAR					(SubstrateData.InlinedBSDF) = Specular;
	SLW_ROUGHNESS					(SubstrateData.InlinedBSDF) = Roughness;
	SLW_TOPMATERIALOPACITY			(SubstrateData.InlinedBSDF) = TopMaterialOpacity;
#if SUBSTRATE_INLINE_SINGLELAYERWATER
	SLW_WATERALBEDO					(SubstrateData.InlinedBSDF) = WaterAlbedo;
	SLW_WATEREXTINCTION				(SubstrateData.InlinedBSDF) = WaterExtinction;
	SLW_WATERPHASEG					(SubstrateData.InlinedBSDF) = WaterPhaseG;
	SLW_COLORSCALEBEHINDWATER		(SubstrateData.InlinedBSDF) = ColorScaleBehindWater;
#endif

#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage = 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= lerp(WaterAlbedo, BaseColor, TopMaterialOpacity);
#endif

	return SubstrateData;
}

FSubstrateData GetSubstrateToonBSDF(
	float ToonProfileId, // in [0, 255]
	float3 BaseColor, float Metallic, float Specular, float Roughness,
	float3 Emissive,
	float2 PatternUVs,
	float Anisotropy, float3 Tangent,
	uint SharedLocalBasisIndex, inout uint SharedLocalBasisTypes)
{
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();

#if MATERIAL_USES_ANISOTROPY && SUBSTRATE_FASTPATH==0
	const uint bHasAnisotropy = Anisotropy != 0 ? 1 : 0;
	if (bHasAnisotropy)
	{
		SubstrateRequestSharedLocalBasisTangent(SharedLocalBasisTypes, SharedLocalBasisIndex);
	}
#else
	const uint bHasAnisotropy = 0;
	Anisotropy = 0.0f;
#endif

#if (SUBSTRATE_FASTPATH==0 || MATERIALBLENDING_ANY_TRANSLUCENT) // !Fast path or rendering translucent materials
	BSDF_SETHASANISOTROPY			(SubstrateData.InlinedBSDF, bHasAnisotropy);
	BSDF_SETISTOPLAYER				(SubstrateData.InlinedBSDF, 1);
#endif

	BSDF_SETTYPE					(SubstrateData.InlinedBSDF, SUBSTRATE_BSDF_TYPE_TOON);
	BSDF_SETSHAREDLOCALBASISID		(SubstrateData.InlinedBSDF, SharedLocalBasisIndex);
	BSDF_SETEMISSIVE				(SubstrateData.InlinedBSDF, Emissive);
	TOON_PROFILEID					(SubstrateData.InlinedBSDF) = ToonProfileId;
	TOON_BASECOLOR					(SubstrateData.InlinedBSDF) = BaseColor;
	TOON_METALLIC					(SubstrateData.InlinedBSDF) = Metallic;
	TOON_SPECULAR					(SubstrateData.InlinedBSDF) = Specular;
	TOON_ROUGHNESS					(SubstrateData.InlinedBSDF) = Roughness;
	TOON_PATTERNUVS					(SubstrateData.InlinedBSDF) = frac(PatternUVs);	// Always apply fract since we cannot store out UVs out of [0,1] range.
	TOON_ANISOTROPY					(SubstrateData.InlinedBSDF) = Anisotropy;

#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage	= 1.0f;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor			= BaseColor;
#endif

	return SubstrateData;
}

///////////////////////////////////////////////////////////////////////////////
// Raytracing material buffer

#ifndef SUBSTRATE_MATERIAL_NUM_UINTS
#error SUBSTRATE_MATERIAL_NUM_UINTS needs to be defined
#endif

/*
Material payload drives whether or not a Substrate material should be fully simplified.
This is setup in a few places:
- FShaderType::ModifyCompilationEnvironment automatically enables SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL for some PayloadType (to automatically impact CHS, MS, RGS and tracing CS)
- When a payload is declared and it needs to transport Substrate data, the size can be computed using Substrate::GetRayTracingMaterialPayloadSizeInBytes, e.g. GetRaytracingMaterialPayloadSizeFullySimplified.
*/

#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#define SUBSTRATE_RT_PAYLOAD_NUM_UINTS SUBSTRATE_FULLY_SIMPLIFIED_NUM_UINTS
#else
// SUBSTRATE_MATERIAL_NUM_UINTS is based on Substrate.BytesPerPixel settings
#define SUBSTRATE_RT_PAYLOAD_NUM_UINTS SUBSTRATE_MATERIAL_NUM_UINTS
#endif

struct FSubstrateRaytracingPayload
{
	// !!! This structure content must be in sync with GetRaytracingMaterialPayloadSize !!!
	SUBSTRATE_TOP_LAYER_TYPE PackedTopLayerData;
	uint Data[SUBSTRATE_RT_PAYLOAD_NUM_UINTS];
};

///////////////////////////////////////////////////////////////////////////////
// Material buffer Addressing

struct FSubstrateAddressing
{
	uint  CurrentIndex;
	uint2 PixelCoords;
	uint  ReadBytes;
};

FSubstrateAddressing GetSubstratePixelDataByteOffset(uint2 PixelPos, uint2 ViewBufferSize, uint SubstrateMaxBytesPerPixel)
{
	FSubstrateAddressing SubstrateAddressing = (FSubstrateAddressing)0;
	SubstrateAddressing.CurrentIndex = 0;
	SubstrateAddressing.PixelCoords = PixelPos;
	SubstrateAddressing.ReadBytes = 0;
	return SubstrateAddressing;
}

///////////////////////////////////////////////////////////////////////////////
// Material buffer Write

// Substrate Rendertarget / UAV setup
// RT0: Color
// [insert here potential velocity rendertarget and prec shadow]
// RT2: Substrate uint
// RT3: Substrate uint
// RT4: Top normal
//
// UAV SSS data written out if needed
// UAV if Substrate data overrun the MRT output size

struct FRWSubstrateMaterialContainerStruct
{
	uint  MaterialRenderTargets[SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT];
	uint2 QuadPixelPos;
	uint  QuadPixelWriteMask;
//	RWTexture2DArray<uint> ExtraMaterialDataUAV; // If this is used, we cannot run shaders on debug...
};

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

#define FRWSubstrateMaterialContainer	FRWSubstrateMaterialContainerStruct
FRWSubstrateMaterialContainer InitialiseRWSubstrateMaterialContainer(RWTexture2DArray<uint> ExtraMaterialDataUAV, uint QuadPixelWriteMask, uint2 QuadPixelPos)
{
	FRWSubstrateMaterialContainer Out;
	UNROLL
	for (int i = 0; i < SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT; ++i)
	{
		Out.MaterialRenderTargets[i] = 0;
	}
	Out.QuadPixelPos = QuadPixelPos;
	Out.QuadPixelWriteMask = QuadPixelWriteMask;
//	Out.ExtraMaterialDataUAV = ExtraMaterialDataUAV;
	return Out;
}

void SubstrateStoreUint1(inout FRWSubstrateMaterialContainer SubstrateBuffer, RWTexture2DArray<uint> ExtraMaterialDataUAV, inout FSubstrateAddressing SubstrateAddressing, uint Data)
{
  #if PLATFORM_SUPPORTS_SUBSTRATE_UINT1
	// We heavily rely on the compiler to optimize out the if code
	if (SubstrateAddressing.CurrentIndex < SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT)
	{
		SubstrateBuffer.MaterialRenderTargets[0] = SubstrateAddressing.CurrentIndex == 0 ? Data : SubstrateBuffer.MaterialRenderTargets[0];
		SubstrateBuffer.MaterialRenderTargets[1] = SubstrateAddressing.CurrentIndex == 1 ? Data : SubstrateBuffer.MaterialRenderTargets[1];
		SubstrateBuffer.MaterialRenderTargets[2] = SubstrateAddressing.CurrentIndex == 2 ? Data : SubstrateBuffer.MaterialRenderTargets[2];
		#if SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT != 3
		#error Substrate SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT has been update but not SubstrateStore function
		#endif
	}
	else if (SubstrateBuffer.QuadPixelWriteMask>0) // Only write data to UAV if pixel is allowed to write (i.e., it is not an helper lane/pixel)
	{
		WriteDataToBuffer(ExtraMaterialDataUAV, Data, uint3(SubstrateAddressing.PixelCoords, SubstrateAddressing.CurrentIndex - SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT), SubstrateBuffer.QuadPixelWriteMask, SubstrateBuffer.QuadPixelPos);
	}
	SubstrateAddressing.CurrentIndex++;
  #endif // PLATFORM_SUPPORTS_SUBSTRATE_UINT1
}

#define SUBSTRATE_STORE_UINT1(x)	SubstrateStoreUint1(SubstrateBuffer, ExtraMaterialDataUAV, SubstrateAddressing, x)

#else // SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

#define FRWSubstrateMaterialContainer FSubstrateRaytracingPayload

FRWSubstrateMaterialContainer InitialiseRWSubstrateMaterialContainer()
{
	FRWSubstrateMaterialContainer Out;
	UNROLL
	for (int i = 0; i < SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT; ++i)
	{
		Out.Data[i] = 0;
	}
	Out.PackedTopLayerData = 0;
	return Out;
}

void SubstrateStoreUint1(inout FRWSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing, uint Data)
{
	BRANCH
	if (SubstrateAddressing.CurrentIndex < SUBSTRATE_RT_PAYLOAD_NUM_UINTS)
	{
		SubstrateBuffer.Data[SubstrateAddressing.CurrentIndex] = Data;
		SubstrateAddressing.CurrentIndex++;
	}
}

#define SUBSTRATE_STORE_UINT1(x) SubstrateStoreUint1(SubstrateBuffer, SubstrateAddressing, x)

#endif // SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

void FinalizeWrites(inout FRWSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing)
{
	// NOP
}

///////////////////////////////////////////////////////////////////////////////
// Material buffer read

#ifndef SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE 
#define SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE 0
#endif

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

#if SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE
#define FSubstrateMaterialContainer RWTexture2DArray<uint>
#else
#define FSubstrateMaterialContainer Texture2DArray<uint>
#endif

uint SubstrateLoadUint1(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing)
{
	uint Data = 0;
  #if PLATFORM_SUPPORTS_SUBSTRATE_UINT1
	#if SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE
	Data = SubstrateBuffer[uint3(SubstrateAddressing.PixelCoords, SubstrateAddressing.CurrentIndex)];
	#else
	const uint MipLevel = 0;
	Data = SubstrateBuffer.mips[MipLevel][uint3(SubstrateAddressing.PixelCoords, SubstrateAddressing.CurrentIndex)];
	#endif
	SubstrateAddressing.CurrentIndex++;
	SubstrateAddressing.ReadBytes += 4;
  #endif
	return Data;
}

uint SubstrateLoadUint1AtIndexOffset(in FSubstrateMaterialContainer SubstrateBuffer, in FSubstrateAddressing SubstrateAddressing, uint IndexOffset)
{
	uint Data = 0;
  #if PLATFORM_SUPPORTS_SUBSTRATE_UINT1
	#if SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE
	Data = SubstrateBuffer[uint3(SubstrateAddressing.PixelCoords, IndexOffset)];
	#else
	const uint MipLevel = 0;
	Data = SubstrateBuffer.mips[MipLevel][uint3(SubstrateAddressing.PixelCoords, IndexOffset)];
	#endif
  #endif
	return Data;
}

// FSubstrateTopLayerDataContainer can be a uint or a uint2 resource, following SUBSTRATE_TOP_LAYER_TYPE according to r.GBufferFormat.
// The type of the texture is set dynamically to avoid compiler warnings and maximize type safety. Note that the SUBSTRATE_TOP_LAYER_TYPE
// define is set at the C++ level so that it is always available, even inside global uniform buffer declarations.
#if SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE
#define FSubstrateTopLayerDataContainer RWTexture2D<SUBSTRATE_TOP_LAYER_TYPE>
#else
#define FSubstrateTopLayerDataContainer Texture2D<SUBSTRATE_TOP_LAYER_TYPE>
#endif

SUBSTRATE_TOP_LAYER_TYPE SubstrateLoadTopLayerData(FSubstrateTopLayerDataContainer InTopLayerContainer, inout FSubstrateAddressing SubstrateAddressing)
{
	#if SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE
	return InTopLayerContainer[SubstrateAddressing.PixelCoords];
	#else
	return InTopLayerContainer.Load(uint3(SubstrateAddressing.PixelCoords, 0));
	#endif
}

#else // SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

#define FSubstrateMaterialContainer FSubstrateRaytracingPayload

uint SubstrateLoadUint1(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing)
{
	BRANCH
	if (SubstrateAddressing.CurrentIndex < SUBSTRATE_RT_PAYLOAD_NUM_UINTS)
	{
		uint Data = SubstrateBuffer.Data[SubstrateAddressing.CurrentIndex];
		SubstrateAddressing.CurrentIndex++;
		SubstrateAddressing.ReadBytes += 4;
		return Data;
	}
	return 0;
}

uint SubstrateLoadUint1AtIndexOffset(in FSubstrateMaterialContainer SubstrateBuffer, in FSubstrateAddressing SubstrateAddressing, uint IndexOffset)
{
	return SubstrateBuffer.Data[IndexOffset];
}

#define FSubstrateTopLayerDataContainer FSubstrateRaytracingPayload

SUBSTRATE_TOP_LAYER_TYPE SubstrateLoadTopLayerData(FSubstrateTopLayerDataContainer Container, inout FSubstrateAddressing SubstrateAddressing)
{
	return Container.PackedTopLayerData;
}

#endif // SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE

///////////////////////////////////////////////////////////////////////////////
// Substrate decal

// Dbuffer support only a single layer, with minimal set of feature (i.e., no anisotropy, ...);
// This special structure is for easing data processing

struct FSubstrateDBuffer
{
	float3 WorldNormal;
	float  Roughness;
	float3 BaseColor;
	float  Specular;
	float3 Emissive;
	float  Metallic;
	float  Coverage;

	// Per-component coverage 
	// Since DBuffer properties are blended independently, they can end-up with varying coverage per-property. 
	// These values are only valid when *reading* DBuffer Value
	float  OneMinusCoverage;
	float  OneMinusCoverage_BaseColor;
	float  OneMinusCoverage_WorldNormal;
	float  OneMinusCoverage_Roughness;
};

void SubstratePackDBuffer(
	in FSubstrateDBuffer In,
	out float4 Out1,
	out float4 Out2,
	out float4 Out3)
{
	// DBuffer packing needs to be blendable
	Out1 = float4(In.BaseColor, In.Coverage);
	Out2 = float4(In.WorldNormal * 0.5f + 128.0f / 255.0f, In.Coverage);
	Out3 = float4(In.Metallic, In.Specular, In.Roughness, In.Coverage);
}

FSubstrateDBuffer SubstrateUnpackDBuffer(
	in float4 In1,
	in float4 In2,
	in float4 In3)
{
	FSubstrateDBuffer Out = (FSubstrateDBuffer)0;

	// Per-component coverage
	Out.OneMinusCoverage			 = (In1.w + In2.w + In3.w) / 3.0f;
	Out.OneMinusCoverage_BaseColor   = In1.w;
	Out.OneMinusCoverage_WorldNormal = In2.w;
	Out.OneMinusCoverage_Roughness   = In3.w;

	// Unpack data
	Out.BaseColor   = In1.xyz;
	Out.Coverage    = 1.f - Out.OneMinusCoverage;
	Out.WorldNormal = In2.rgb * 2 - (256.0 / 255.0);
	Out.Metallic    = In3.x;
	Out.Specular    = In3.y;
	Out.Roughness   = In3.z;
	Out.Emissive    = 0.f;


	return Out;
}

///////////////////////////////////////////////////////////////////////////////
// Integration settings

struct FSubstrateIntegrationSettings
{
	bool bForceFullyRough;
	bool bForceFullyRoughAccurate;
	bool bRoughDiffuseEnabled;
	bool bRoughnessTracking;
	bool bUseShadowTermHairTransmittance;

	// Embbeded debug settings
	int  SliceStoringDebugSubstrateTreeData;
	int  PeelLayersAboveDepth;
};

FSubstrateIntegrationSettings InitSubstrateIntegrationSettings(bool bForceFullyRough, bool bRoughDiffuseEnabled, int PeelLayersAboveDepth, bool bRoughnessTracking)
{
	FSubstrateIntegrationSettings Out;
	Out.bForceFullyRough = bForceFullyRough;
	Out.bForceFullyRoughAccurate = false;
	Out.bRoughDiffuseEnabled = bRoughDiffuseEnabled;
	Out.bRoughnessTracking = bRoughnessTracking;
	Out.bUseShadowTermHairTransmittance = false;

	Out.PeelLayersAboveDepth = PeelLayersAboveDepth;
	Out.SliceStoringDebugSubstrateTreeData = -1;
	return Out;
}

FSubstrateIntegrationSettings InitSubstrateIntegrationSettings()
{
	return InitSubstrateIntegrationSettings(false, true, -1, false);
}

///////////////////////////////////////////////////////////////////////////////
// Shared local bases

#define SUBSTRATE_BASIS_TYPE_NORMAL 0u
#define SUBSTRATE_BASIS_TYPE_TANGENT 1u
struct FSharedLocalBases
{
	uint Count;
	uint Types;
	float3 Normals[SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS];	// once registered, normals are always world space
	float3 Tangents[SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS];// idem for tangents
};

FSharedLocalBases SubstrateInitialiseSharedLocalBases()
{
	FSharedLocalBases SharedLocalBases = (FSharedLocalBases)0;
	return SharedLocalBases;
}

void SubstrateRequestSharedLocalBasisTangent(inout uint Types, uint Index)
{
	Types = Types | (SUBSTRATE_BASIS_TYPE_TANGENT << Index);
}

uint SubstrateGetSharedLocalBasisType(in uint Types, uint Index)
{
	return (Types >> Index) & 0x1 ? SUBSTRATE_BASIS_TYPE_TANGENT : SUBSTRATE_BASIS_TYPE_NORMAL;
}

///////////////////////////////////////////////////////////////////////////////
// Header

struct FSubstratePixelHeader
{
	uint ClosureCount;

	uint State;

#if SUBSTRATE_INLINE_SHADING
	// All the shared local bases in VGPRS. This is only used in basepass and during forward rendering.
	FSharedLocalBases SharedLocalBases;
	// The Substrate material BSDF topology organised as a tree
	FSubstrateTree SubstrateTree;
	FSubstrateIrradianceAndOcclusion IrradianceAO;

	// Generated according to the compiled material. It is convenient to have them here in order to avoid expenssive compilation resulting from isolated function taking as input large strucutre using inout qualifier.
	void PreUpdateAllBSDFWithBottomUpOperatorVisit(float3 V);
	void UpdateAllBSDFsOperatorCoverageTransmittance(FSubstrateIntegrationSettings Settings, float3 V);
	void UpdateAllOperatorsCoverageTransmittance();
	void UpdateAllBSDFWithBottomUpOperatorVisit();
	void PreUpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified(float3 V);
	void UpdateAllBSDFsOperatorCoverageTransmittance_FullySimplified(FSubstrateIntegrationSettings Settings, float3 V);
	void UpdateAllOperatorsCoverageTransmittance_FullySimplified();
	void UpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified();

	void SubstrateUpdateTree(
		float3 V,
		FSubstrateIntegrationSettings Settings);

	void SubstrateUpdateTree(
		in FSubstrateData SubstrateData,
		float3 V,
		FSubstrateIntegrationSettings Settings,
		inout float OutCoverage,
		inout float3 OutTransmittancePreCoverage);

#if SUBSTRATE_OPTIMIZED_UNLIT
	void SubstrateUpdateTreeUnlit(
		in		uint2	PixelPos,
		in		float3	V,
		in		FSubstrateData SubstrateData,
		inout	float3	OutEmissiveLuminance,
		inout	float	OutDualBlendSurfaceCoverage,
		inout	float3	OutDualBlendSurfaceTransmittancePreCoverage,
		inout	float3	OutNormal)
	{
		FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, false /*SubstrateStruct.bRoughDiffuse*/, -1/*SubstrateStruct.PeelLayersAboveDepth*/, false/*SubstrateStruct.bRoughnessTracking*/);
		this.SubstrateUpdateTree(SubstrateData, V, Settings, OutDualBlendSurfaceCoverage, OutDualBlendSurfaceTransmittancePreCoverage);

		FSubstrateBSDF UnlitBSDF = this.SubstrateTree.BSDFs[0];
		OutEmissiveLuminance = BSDF_GETEMISSIVE(UnlitBSDF) * UnlitBSDF.LuminanceWeightV;
		OutNormal = UNLIT_NORMAL(UnlitBSDF);
	}
	//SUBSTRATE_TODO version with emissive only
#endif // SUBSTRATE_OPTIMIZED_UNLIT

#endif

#if SUBSTRATE_DEFERRED_SHADING
	FSubstrateMaterialContainer SubstrateBuffer;
	uint SharedLocalBasesIndexOffset;
	uint PackedHeader;	// Used for special simple/eye/etc. packing, or for complex to recover LocalBasesTypes, or use GetLocalBasesCount in rare cases (VSM, RTShadow or visualisation).
	SUBSTRATE_TOP_LAYER_TYPE PackedTopLayerData;
#endif

	// Decals
	FSubstrateDBuffer SubstrateConvertToDBuffer(in FSubstrateBSDF InBSDF);
	void SubstrateConvertFromDBuffer(in FSubstrateDBuffer In, inout FSubstrateData OutData);

	// Material mode & BSDF type
	uint GetMaterialMode()										{ return (this.State & HEADER_MASK_MATERIALMODE); }
	#if SUBSTRATE_USES_FORCED_COMPLEXITY
	bool IsSimpleMaterial()										{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_SIMPLE || SUBSTRATE_FASTPATH; }
	bool IsSingleMaterial()										{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_SINGLE || SUBSTRATE_SINGLEPATH; }
	bool IsComplexMaterial()									{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_COMPLEX && !SUBSTRATE_FASTPATH && !SUBSTRATE_SINGLEPATH; }
	#else
	bool IsSimpleMaterial()										{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_SIMPLE; }
	bool IsSingleMaterial()										{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_SINGLE; }
	bool IsComplexMaterial()									{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLAB_COMPLEX; }
	#endif
	bool IsComplexSpecialMaterial()								{ return HEADER_GETCOMPLEXPATHMODE(this.State) == COMPLEXPATH_MODE_SLABSPECIAL; }
	bool IsSingleLayerWater()									{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_SLWATER; }
	bool IsHair()												{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_HAIR; }
	bool IsEye()												{ return (this.State & HEADER_MASK_MATERIALMODE) == HEADER_MATERIALMODE_EYE; }
	uint SubstrateGetBSDFType()
	{
		switch (this.GetMaterialMode())
		{
		case HEADER_MATERIALMODE_SLAB_SIMPLE:
		case HEADER_MATERIALMODE_SLAB_SINGLE:
		case HEADER_MATERIALMODE_SLAB_COMPLEX:	return HEADER_GETCOMPLEXPATHMODE(this.State) == COMPLEXPATH_MODE_TOON ? SUBSTRATE_BSDF_TYPE_TOON : SUBSTRATE_BSDF_TYPE_SLAB; // Get the general type for unpacking. So far, SUBSTRATE_BSDF_TYPE_SLAB that can also be SUBSTRATE_BSDF_TYPE_TOON
		case HEADER_MATERIALMODE_SLWATER:		return SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
		case HEADER_MATERIALMODE_HAIR:			return SUBSTRATE_BSDF_TYPE_HAIR;
		case HEADER_MATERIALMODE_EYE:			return SUBSTRATE_BSDF_TYPE_EYE;
		default:								return SUBSTRATE_BSDF_TYPE_SLAB;
		}
	}

	bool HasPrecShadowMask()									{ return (this.State & HEADER_MASK_HASPRECSHADOWMASK) != 0; }
#if ALLOW_STATIC_LIGHTING
	bool HasZeroPrecShadowMask()								{ return (this.State & HEADER_MASK_ZEROPRECSHADOWMASK) != 0; }
	bool IsFirstPerson()										{ return false; }
#else
	bool HasZeroPrecShadowMask()								{ return false; }
	bool IsFirstPerson()										{ return (this.State & HEADER_MASK_ISFIRSTPERSON) != 0; }
#endif
	bool DoesCastContactShadow()								{ return (this.State & HEADER_MASK_CASTCONTACTSHADOW) != 0; }
	bool HasSubsurface()										{ return (this.State & HEADER_MASK_HASSUBSURFACE) != 0; }
	bool HasDynamicIndirectShadowCasterRepresentation()			{ return (this.State & HEADER_MASK_HASDYNINDIRECTSHADOWCASTER) != 0; }

	void SetHasPrecShadowMask(bool bIn)							{ this.State |= (bIn ? HEADER_MASK_HASPRECSHADOWMASK : 0u); }
#if ALLOW_STATIC_LIGHTING
	void SetZeroPrecShadowMask(bool bIn)						{ this.State |= (bIn ? HEADER_MASK_ZEROPRECSHADOWMASK : 0u); }
	void SetIsFirstPerson(bool bIn)								{ }
#else
	void SetZeroPrecShadowMask(bool bIn)						{ }
	void SetIsFirstPerson(bool bIn)								{ this.State |= (bIn ? HEADER_MASK_ISFIRSTPERSON : 0u); }
#endif
	void SetCastContactShadow(bool bIn)							{ this.State  = (this.State & ~HEADER_MASK_CASTCONTACTSHADOW) | (bIn ? HEADER_MASK_CASTCONTACTSHADOW : 0u); }
	void SetHasSubsurface(bool bIn)								{ this.State |= (bIn ? HEADER_MASK_HASSUBSURFACE : 0u); }
	void SetDynamicIndirectShadowCasterRepresentation(bool bIn)	{ this.State |= (bIn ? HEADER_MASK_HASDYNINDIRECTSHADOWCASTER : 0u); }

	bool IsSubstrateMaterial()									{ return this.GetMaterialMode() != HEADER_MATERIALMODE_NONE; }
	void SetMaterialMode(uint MaterialMode)						{ this.State = (this.State & ~HEADER_MASK_MATERIALMODE) | (MaterialMode & HEADER_MASK_MATERIALMODE); }

	uint GetComplexPathMode()									{ return HEADER_GETCOMPLEXPATHMODE(this.State); }
	void SetComplexPathMode(uint ComplexPathMode)				{ HEADER_SETCOMPLEXPATHMODE(this.State, ComplexPathMode); }

#if SUBSTRATE_DEFERRED_SHADING
	uint GetLocalBasesCount()									{ return GetMaterialMode() == HEADER_MATERIALMODE_SLAB_COMPLEX ? HEADER_GETSHAREDLOCALBASESCOUNT(PackedHeader) : 1; }
#endif
};

#if SUBSTRATE_DEFERRED_SHADING && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
// Some compiler complains that FSubstratePixelHeader is not completely initialised, Initialise with dummy resource seems to work when unused in code.
FSubstrateMaterialContainer DummySubstrateBuffer;
#endif

FSubstratePixelHeader InitialiseSubstratePixelHeader()
{
	FSubstratePixelHeader Out;
	Out.ClosureCount = 0;
#if SUBSTRATE_INLINE_SHADING
	Out.SharedLocalBases = (FSharedLocalBases)0;
	Out.SubstrateTree = GetInitialisedSubstrateTree();
	Out.IrradianceAO = InitIrradianceAndOcclusion();
#endif
#if SUBSTRATE_DEFERRED_SHADING
#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	Out.SubstrateBuffer = DummySubstrateBuffer;
#endif
	Out.SharedLocalBasesIndexOffset = 0;
	Out.PackedHeader = 0;
	Out.PackedTopLayerData = 0;
#endif
	Out.State = 0;
	Out.SetMaterialMode(HEADER_MATERIALMODE_SLAB_SIMPLE); // Should be a NOP since SIMPLE=0. This is for non Substrate material or sky pixels to be considered as simple (so that (Substrate simple pixels) + (non-Substrate pixels) can build a simple tile).
	return Out;
}

FSubstrateDBuffer FSubstratePixelHeader::SubstrateConvertToDBuffer(
	in FSubstrateBSDF InBSDF)
{
	FSubstrateDBuffer Out = (FSubstrateDBuffer)0;

	#if SUBSTRATE_INLINE_SHADING

	const float3 DiffuseColor = SLAB_DIFFUSEALBEDO(InBSDF);
	const float3 F0 = SLAB_F0(InBSDF);

	Out.Roughness = SLAB_ROUGHNESS(InBSDF);
	Out.Metallic = F0RGBToMetallic(F0);
	Out.Specular = F0RGBToDielectricSpecular(F0);
	Out.BaseColor = lerp(DiffuseColor, F0, Out.Metallic);
	Out.Emissive = BSDF_GETEMISSIVE(InBSDF);
	Out.WorldNormal = this.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(InBSDF)];
	Out.Coverage = InBSDF.Coverage;

	#else
	Out.WorldNormal = float3(0,0,1);
	Out.Coverage = 1.f;
	#endif
	return Out;
}

void FSubstratePixelHeader::SubstrateConvertFromDBuffer(
	in FSubstrateDBuffer In,
	inout FSubstrateData OutData)
{
	// Header
	const uint SharedLocalBasisIndex = 0;
	uint SharedLocalBasisTypes = 0;
	this.ClosureCount = 1;
	#if SUBSTRATE_INLINE_SHADING
	this.SharedLocalBases = SubstrateInitialiseSharedLocalBases();
	this.SharedLocalBases.Normals[SharedLocalBasisIndex] = In.WorldNormal;
	this.SharedLocalBases.Tangents[SharedLocalBasisIndex] = 0.f;
	this.IrradianceAO = InitIrradianceAndOcclusion();
	#endif
	this.State = 0;
	this.SetMaterialMode(HEADER_MATERIALMODE_SLAB_SIMPLE);

	// Decal only support a single Slab BSDF with no-anisotropy
	const FSubstratePixelFootprint PixelFootprint = (FSubstratePixelFootprint)0;
	const float DefaultThicknessCm = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
	OutData = GetSubstrateSlabBSDF(
		PixelFootprint,		// PixelFootprint
		In.WorldNormal,
		ComputeDiffuseAlbedo(In.BaseColor, In.Metallic),
		ComputeF0(half(In.Specular), half3(In.BaseColor), half(In.Metallic)),
		1.0f,				// F90
		In.Roughness,		// Roughness
		0.f,				// Anisotropy
		0,					// SSSProfileID
		false, 				// bSupportDefaultSSSProfile
		0.f,				// SSSMFP
		0.f,				// SSSMFPScale
		0.f,				// SSSPhaseAniso
		SSS_TYPE_NONE,		// UseSSSDiffusion
		0.f,				// Emissive
		0.f,				// SecondRoughness
		0.f,				// SecondRoughnessWeight
		0.f,				// SecondRoughnessAsSimpleClearCoat
		0.f,				// ClearCoatUseSecondNormal
		0.f,				// ClearCoatBottomNormal
		0.f,				// FuzzAmount
		0.f,				// FuzzColor
		In.Roughness,		// FuzzRoughness
		1.f,				// GlintValue, 
		(GLINT_UV_TYPE)0,	// GlintUV,
		0.f, 				// SpecularProfileId
		DefaultThicknessCm,
		false,				// bIsThinSurface,
		false,				// bIsAtTheBottomOfTopology, this will disable many unneeded features.
		SharedLocalBasisIndex,
		SharedLocalBasisTypes);
	#if SUBSTRATE_INLINE_SHADING
	OutData.InlinedBSDF.Emissive = In.Emissive;
	OutData.InlinedBSDF.Coverage = In.Coverage;
	#endif
}

float3x3 SubstrateGetBSDFSharedBasis_InlineShading(in FSubstratePixelHeader SubstratePixelHeader, in FSubstrateBSDF BSDF)
{
#if SUBSTRATE_INLINE_SHADING
	// Assume Normal & Tangent are already been normalized
	float3 Normal  = SubstratePixelHeader.SharedLocalBases.Normals[BSDF_GETSHAREDLOCALBASISID(BSDF)];
	float3 Tangent = SubstratePixelHeader.SharedLocalBases.Tangents[BSDF_GETSHAREDLOCALBASISID(BSDF)];
	float3 Bitangent = cross(Normal, Tangent);
	return float3x3(Tangent, Bitangent, Normal);
#else
	return float3x3(float3(1,0,0), float3(0,1,0), float3(0,0,1));
#endif
}

float3x3 SubstrateGetBSDFSharedBasis_DeferredShading(in FSubstratePixelHeader SubstratePixelHeader, in FSubstrateBSDF BSDF, in FSubstrateAddressing SubstrateAddressing)
{
#if SUBSTRATE_DEFERRED_SHADING
	if (SubstratePixelHeader.IsSimpleMaterial() || SubstratePixelHeader.IsSingleMaterial() || SubstratePixelHeader.IsEye() || SubstratePixelHeader.IsHair() || SubstratePixelHeader.IsSingleLayerWater())
	{		
		return GetTangentBasis(SubstrateUnpackTopLayerData(SubstratePixelHeader.PackedTopLayerData).WorldNormal);
	}

	uint PackedBasis = SubstrateLoadUint1AtIndexOffset(SubstratePixelHeader.SubstrateBuffer, SubstrateAddressing, SubstratePixelHeader.SharedLocalBasesIndexOffset + BSDF_GETSHAREDLOCALBASISID(BSDF));

	uint BasisType   = SubstrateGetSharedLocalBasisType(HEADER_GETSHAREDLOCALBASESTYPE(SubstratePixelHeader.PackedHeader), BSDF_GETSHAREDLOCALBASISID(BSDF));
	float3x3 OutTangentBasis;
	if (BasisType == SUBSTRATE_BASIS_TYPE_NORMAL)
	{
		float3 Normal = SubstrateUnpackNormal(PackedBasis);
		OutTangentBasis = GetTangentBasis(Normal);
	}
	else // if (BasisType == SUBSTRATE_BASIS_TYPE_TANGENT)
	{
		float3 Normal;
		float3 Tangent;
		SubstrateUnpackNormalAndTangent(Normal, Tangent, PackedBasis);

		OutTangentBasis[0] = Tangent;
		OutTangentBasis[1] = cross(Normal, Tangent);
		OutTangentBasis[2] = Normal;
	}
	return OutTangentBasis;
#else
	return float3x3(float3(1,0,0), float3(0,1,0), float3(0,0,1));
#endif
}

float3x3 SubstrateGetBSDFSharedBasis(in FSubstratePixelHeader SubstratePixelHeader, in FSubstrateBSDF BSDF, in FSubstrateAddressing SubstrateAddressing)
{
#if SUBSTRATE_INLINE_SHADING
	return SubstrateGetBSDFSharedBasis_InlineShading(SubstratePixelHeader, BSDF);
#else 
	return SubstrateGetBSDFSharedBasis_DeferredShading(SubstratePixelHeader, BSDF, SubstrateAddressing);
#endif
}

uint SubstrateGetBSDFBasisType(in FSubstratePixelHeader SubstratePixelHeader, in FSubstrateBSDF BSDF)
{
	uint BasisType = SUBSTRATE_BASIS_TYPE_NORMAL;
	#if SUBSTRATE_DEFERRED_SHADING
	const bool bHasNormalLayer = SubstratePixelHeader.IsSimpleMaterial() || SubstratePixelHeader.IsSingleMaterial() || SubstratePixelHeader.IsEye() || SubstratePixelHeader.IsHair() || SubstratePixelHeader.IsSingleLayerWater();
	if (!bHasNormalLayer)
	{
		BasisType = SubstrateGetSharedLocalBasisType(HEADER_GETSHAREDLOCALBASESTYPE(SubstratePixelHeader.PackedHeader), BSDF_GETSHAREDLOCALBASISID(BSDF));
	}
	#endif
	return BasisType;
}

///////////////////////////////////////////////////////////////////////////////
// Operator nodes

FSubstrateData FSubstrateTree::SubstrateAdd(FSubstrateData A, FSubstrateData B, int OperatorIndex, uint MaxDistanceFromLeaves)
{
	#if ALLOW_SUBSTRATE_TREE
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();
	SubstrateData.OperatorIndex = OperatorIndex;
#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor = A.PreviewColor + B.PreviewColor;
#endif

	OperatorCount++;
	Operators[OperatorIndex]						= GetInitialisedSubstrateOperator();
	Operators[OperatorIndex].MaxDistanceFromLeaves	= MaxDistanceFromLeaves;
	Operators[OperatorIndex].Type					= SUBSTRATE_OPERATOR_ADD;
	Operators[OperatorIndex].LeftIndex				= A.OperatorIndex;
	Operators[OperatorIndex].RightIndex				= B.OperatorIndex;

	Operators[A.OperatorIndex].ParentIndex			= OperatorIndex;
	Operators[B.OperatorIndex].ParentIndex			= OperatorIndex;

	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

FSubstrateData FSubstrateTree::SubstrateWeight(FSubstrateData A, float Weight, int OperatorIndex, uint MaxDistanceFromLeaves)
{
	#if ALLOW_SUBSTRATE_TREE
	const float SafeWeight = saturate(Weight);

	FSubstrateData SubstrateData = GetInitialisedSubstrateData();
	SubstrateData.OperatorIndex = OperatorIndex;
#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor = A.PreviewColor * SafeWeight;
#endif

	OperatorCount++;
	Operators[OperatorIndex]						= GetInitialisedSubstrateOperator();
	Operators[OperatorIndex].MaxDistanceFromLeaves	= MaxDistanceFromLeaves;
	Operators[OperatorIndex].Type					= SUBSTRATE_OPERATOR_WEIGHT;
	Operators[OperatorIndex].Weight					= SafeWeight;
	Operators[OperatorIndex].LeftIndex				= A.OperatorIndex;

	Operators[A.OperatorIndex].ParentIndex			= OperatorIndex;

	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

FSubstrateData FSubstrateTree::SubstrateHorizontalMixing(FSubstrateData Background, FSubstrateData Foreground, float Mix, int OperatorIndex, uint MaxDistanceFromLeaves)
{
	#if ALLOW_SUBSTRATE_TREE
	const float ForegroundMixFactor = saturate(Mix);
	const float BackgroundMixFactor = 1.0 - ForegroundMixFactor;

	FSubstrateData SubstrateData = GetInitialisedSubstrateData();
	SubstrateData.OperatorIndex = OperatorIndex;
#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor = lerp(Background.PreviewColor, Foreground.PreviewColor, Mix);
#endif

	OperatorCount++;
	Operators[OperatorIndex]						= GetInitialisedSubstrateOperator();
	Operators[OperatorIndex].MaxDistanceFromLeaves	= MaxDistanceFromLeaves;
	Operators[OperatorIndex].Type					= SUBSTRATE_OPERATOR_HORIZONTAL;
	Operators[OperatorIndex].Weight					= Mix;
	Operators[OperatorIndex].LeftIndex				= Background.OperatorIndex;
	Operators[OperatorIndex].RightIndex				= Foreground.OperatorIndex;

	Operators[Background.OperatorIndex].ParentIndex	= OperatorIndex;
	Operators[Foreground.OperatorIndex].ParentIndex	= OperatorIndex;

	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

FSubstrateData FSubstrateTree::SubstrateVerticalLayering(FSubstrateData Top, FSubstrateData Base, int OperatorIndex, uint MaxDistanceFromLeaves)
{
	#if ALLOW_SUBSTRATE_TREE
	FSubstrateData SubstrateData = GetInitialisedSubstrateData();
	SubstrateData.OperatorIndex = OperatorIndex;
#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor = lerp(Top.PreviewColor, Base.PreviewColor, 0.5);
#endif

	OperatorCount++;
	Operators[OperatorIndex]						= GetInitialisedSubstrateOperator();
	Operators[OperatorIndex].MaxDistanceFromLeaves	= MaxDistanceFromLeaves;
	Operators[OperatorIndex].Type					= SUBSTRATE_OPERATOR_VERTICAL;
	Operators[OperatorIndex].LeftIndex				= Top.OperatorIndex;
	Operators[OperatorIndex].RightIndex				= Base.OperatorIndex;

	Operators[Top.OperatorIndex].ParentIndex		= OperatorIndex;
	Operators[Base.OperatorIndex].ParentIndex		= OperatorIndex;

	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

// Take the inlined BSDF resulting from parameter blending (or BSDF creation) and register it to the SubstrateTree
FSubstrateData FSubstrateTree::PromoteParameterBlendedBSDFToOperator(FSubstrateData SubstrateData, int OperatorIndex, int BSDFIndex, int LayerDepth, int bIsBottom)
{
	#if ALLOW_SUBSTRATE_TREE
	BSDFCount++;
	OperatorCount++;

	BSDFs[BSDFIndex] = SubstrateData.InlinedBSDF;
	
	Operators[OperatorIndex]						= GetInitialisedSubstrateOperator();
	Operators[OperatorIndex].LayerDepth				= LayerDepth;
	Operators[OperatorIndex].MaxDistanceFromLeaves	= 0;
	Operators[OperatorIndex].Type					= SUBSTRATE_OPERATOR_BSDF;
	Operators[OperatorIndex].LeftIndex				= BSDFIndex;

	BSDFs[BSDFIndex].OperatorIndex					= OperatorIndex;
	BSDFs[BSDFIndex].bIsBottom						= bIsBottom;
	BSDFs[BSDFIndex].bIsTop							= LayerDepth == 0;

	SubstrateData.OperatorIndex = OperatorIndex;
	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

float HorizontalMixingParameterBlendingBSDFCoverageToNormalMix(FSubstrateData Background, FSubstrateData Foreground, float HorizontalMixValue)
{
	// This is a normalised mix so even though both BSDF have weights, the sum of weights used to combined them should always be 1.
	// This also account for relative BSDF coverage.
#if SUBSTRATE_INLINE_SHADING
	const float SafeBackCoverage = saturate(Background.InlinedBSDF.Coverage);
	const float SafeForeCoverage = saturate(Foreground.InlinedBSDF.Coverage);
	const float ForegroundMixFactor = saturate(HorizontalMixValue);
	const float BackgroundMixFactor = 1.0 - ForegroundMixFactor;

	return (SafeForeCoverage * ForegroundMixFactor) / max(SUBSTRATE_EPSILON, SafeForeCoverage * ForegroundMixFactor + SafeBackCoverage * BackgroundMixFactor);
#else
	return 0.5f;
#endif
}

// Merge two type of SSS type, by using the most complex behavior, i.e., No < Wrap < Diffusion < Diffusion Profile
uint SubstrateMergeSSSType(uint SSSType0, uint SSSType1)
{
	return max(SSSType0, SSSType1);
}

// Note about parameter blending
//  - We can only parameter blend Slab nodes. This error is handled in the compiler.
//  - We can only parameter blend one BSDF into another single BSDF. More complex topology are not handled and result in a compiler error. This could be extended to a single BSDF per layer.
//  - Maybe at some point we can propose a concatenate node, compressing a complex topology into a single one.
//  - Slab node with SSSprofile cannot be blendend. Those will have to be specifically handled by the user and not mixed with any other slab if it can be demoted to parameter blending

// ==> NOTE: Always pair with the compiler behavior in SubstrateCompilationInfoHorizontalMixingParamBlend
FSubstrateData SubstrateHorizontalMixingParameterBlending(FSubstrateData Background, FSubstrateData Foreground, float HorizontalMixValue, float NormalMixCodeChunk, uint NewNormalIndex, inout uint OutSharedLocalBasisTypes, float BackgroundNoV, float ForegroundNoV)
{
	#if ALLOW_SUBSTRATE_TREE
	const float ForegroundOtherMixFactor = saturate(NormalMixCodeChunk);	// Horizontal mixing value combined with coverage
	const float ForegroundLightingMixFactor = saturate(HorizontalMixValue);

	FSubstrateData Result = GetInitialisedSubstrateData();
#define ResultBSDF		Result.InlinedBSDF
#define ForegroundBSDF	Foreground.InlinedBSDF
#define BackgroundBSDF	Background.InlinedBSDF

	// We can only parameter blend the slab 
	BSDF_SETTYPE(ResultBSDF,			SUBSTRATE_BSDF_TYPE_SLAB);
	BSDF_SETSHAREDLOCALBASISID(ResultBSDF,NewNormalIndex);

	BSDF_SETISTOPLAYER(ResultBSDF,		0);
	BSDF_SETHASANISOTROPY(ResultBSDF,	BSDF_GETHASANISOTROPY(ForegroundBSDF)			| BSDF_GETHASANISOTROPY(BackgroundBSDF));
	BSDF_SETSSSTYPE(ResultBSDF,			SubstrateMergeSSSType(BSDF_GETSSSTYPE(ForegroundBSDF), BSDF_GETSSSTYPE(BackgroundBSDF)));
	BSDF_SETISTHIN(ResultBSDF,			BSDF_GETISTHIN(ForegroundBSDF)					| BSDF_GETISTHIN(BackgroundBSDF));
	BSDF_SETHASMFP(ResultBSDF,			BSDF_GETHASMFP(ForegroundBSDF)					| BSDF_GETHASMFP(BackgroundBSDF));
	BSDF_SETHASHAZINESS(ResultBSDF,		BSDF_GETHASHAZINESS(ForegroundBSDF)				| BSDF_GETHASHAZINESS(BackgroundBSDF));
	BSDF_SETHASF90(ResultBSDF,			BSDF_GETHASF90(ForegroundBSDF)					| BSDF_GETHASF90(BackgroundBSDF));
	BSDF_SETHASFUZZ(ResultBSDF,			BSDF_GETHASFUZZ(ForegroundBSDF)					| BSDF_GETHASFUZZ(BackgroundBSDF));
	BSDF_SETHASTRANSABOVE(ResultBSDF,	BSDF_GETHASTRANSABOVE(ForegroundBSDF)			| BSDF_GETHASTRANSABOVE(BackgroundBSDF));

	// Bake the F90 into F0 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
	BackgroundBSDF.SubstrateBakeF90ToReflectivity(BackgroundNoV);
	ForegroundBSDF.SubstrateBakeF90ToReflectivity(ForegroundNoV);
	BSDF_SETHASF90(ResultBSDF, 0);

	// Bake the Specular Profiles into F0/F90 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
#if SUBSTRATE_SPECPROFILE_ENABLED
	BackgroundBSDF.SubstrateBakeSpecularProfileToReflectivity(BackgroundNoV);
	ForegroundBSDF.SubstrateBakeSpecularProfileToReflectivity(ForegroundNoV);
	BSDF_SETHASSPECPROFILE(ResultBSDF, 0);
#endif

#if SUBSTRATE_GLINTS_ENABLED 
	ForegroundBSDF.SubstrateBakeGlintToReflectivity();
	BackgroundBSDF.SubstrateBakeGlintToReflectivity();
	BSDF_SETHASGLINT(ResultBSDF, 0);
#endif // SUBSTRATE_GLINTS_ENABLED

	BSDF_SETEMISSIVE(ResultBSDF,		lerp(BSDF_GETEMISSIVE(BackgroundBSDF),			BSDF_GETEMISSIVE(ForegroundBSDF),			ForegroundOtherMixFactor));
	SLAB_DIFFUSEALBEDO(ResultBSDF) =	lerp(SLAB_DIFFUSEALBEDO(BackgroundBSDF),		SLAB_DIFFUSEALBEDO(ForegroundBSDF),			ForegroundOtherMixFactor);
	SLAB_F0(ResultBSDF) =				lerp(SLAB_F0(BackgroundBSDF),					SLAB_F0(ForegroundBSDF),					ForegroundOtherMixFactor);
	SLAB_F90(ResultBSDF) =				lerp(SLAB_F90(BackgroundBSDF),					SLAB_F90(ForegroundBSDF),					ForegroundOtherMixFactor);
	SLAB_FUZZ_COLOR(ResultBSDF)	=		lerp(SLAB_FUZZ_COLOR(BackgroundBSDF),			SLAB_FUZZ_COLOR(ForegroundBSDF),			ForegroundOtherMixFactor);

	BSDF_SETTHICKNESSCM(ResultBSDF,		lerp(BSDF_GETTHICKNESSCM(BackgroundBSDF),		BSDF_GETTHICKNESSCM(ForegroundBSDF),		ForegroundOtherMixFactor));
	SLAB_ROUGHNESS(ResultBSDF) =		lerp(SLAB_ROUGHNESS(BackgroundBSDF),			SLAB_ROUGHNESS(ForegroundBSDF),				ForegroundOtherMixFactor);
	SLAB_ANISOTROPY(ResultBSDF) =		lerp(SLAB_ANISOTROPY(BackgroundBSDF),			SLAB_ANISOTROPY(ForegroundBSDF),			ForegroundOtherMixFactor);
	SLAB_FUZZ_AMOUNT(ResultBSDF) =		lerp(SLAB_FUZZ_AMOUNT(BackgroundBSDF),			SLAB_FUZZ_AMOUNT(ForegroundBSDF),			ForegroundOtherMixFactor);
	SLAB_FUZZ_ROUGHNESS(ResultBSDF)	=	lerp(SLAB_FUZZ_ROUGHNESS(BackgroundBSDF),		SLAB_FUZZ_ROUGHNESS(ForegroundBSDF),		ForegroundOtherMixFactor);
	SLAB_HAZINESS(ResultBSDF) =			LerpHazinessParameterBlending(BackgroundBSDF,	ForegroundBSDF,								ForegroundOtherMixFactor);

	if (BSDF_GETSSSTYPE(ResultBSDF) == SSS_TYPE_DIFFUSION_PROFILE)
	{
		const bool bForegroundHasSSSProfile = BSDF_GETSSSTYPE(ForegroundBSDF) == SSS_TYPE_DIFFUSION_PROFILE;
		const bool bBackgroundHasSSSProfile = BSDF_GETSSSTYPE(BackgroundBSDF) == SSS_TYPE_DIFFUSION_PROFILE;

		// As mentioned above, we cannot blend SSSProfiles.
		if (bForegroundHasSSSProfile && bBackgroundHasSSSProfile)
		{
			// Select the profile that has the highest mixing weight.
			SLAB_SSSPROFILEID(ResultBSDF) = ForegroundLightingMixFactor > 0.5f ? SLAB_SSSPROFILEID(ForegroundBSDF) : SLAB_SSSPROFILEID(BackgroundBSDF);
		}
		else
		{
			// Select the only profile available.
			SLAB_SSSPROFILEID(ResultBSDF) = bForegroundHasSSSProfile ? SLAB_SSSPROFILEID(ForegroundBSDF) : SLAB_SSSPROFILEID(BackgroundBSDF);
		}

		// We also lerp the radius scale, while taking into account whether or not the one or the other have SSSProfile.
		SLAB_SSSPROFILERADIUSSCALE(ResultBSDF) = lerp(
			bBackgroundHasSSSProfile ? SLAB_SSSPROFILERADIUSSCALE(BackgroundBSDF) : 0.0f,
			bForegroundHasSSSProfile ? SLAB_SSSPROFILERADIUSSCALE(ForegroundBSDF) : 0.0f,
			ForegroundOtherMixFactor);

		SLAB_SSSPROFILETHICKNESSCM(ResultBSDF) = bForegroundHasSSSProfile ? BSDF_GETTHICKNESSCM(ForegroundBSDF) : BSDF_GETTHICKNESSCM(BackgroundBSDF);

		if (SLAB_SSSPROFILERADIUSSCALE(ResultBSDF) == 0.0f)
		{
			SLAB_SSSPROFILEID(ResultBSDF) = SSS_PROFILE_ID_INVALID;
			BSDF_SETSSSTYPE(ResultBSDF, SSS_TYPE_NONE);
		}
	}
	else
	{
		SLAB_SSSMFP(ResultBSDF) =				lerp(SLAB_SSSMFP(BackgroundBSDF),				SLAB_SSSMFP(ForegroundBSDF),				ForegroundOtherMixFactor);
		SLAB_SSSPHASEANISOTROPY(ResultBSDF) =	lerp(SLAB_SSSPHASEANISOTROPY(BackgroundBSDF),	SLAB_SSSPHASEANISOTROPY(ForegroundBSDF),	ForegroundOtherMixFactor);

		// We still try to disable SSS if possible: for performance and also to preserve the surface color (especially specular reflection color).
		if (all(SLAB_SSSMFP(ResultBSDF) == 0.0f))
		{
			BSDF_SETSSSTYPE(ResultBSDF, SSS_TYPE_NONE);
		}
	}

	// Request a tangent basis if the blended BSDF has anisotropy
	if (!SUBSTRATE_FASTPATH && BSDF_GETHASANISOTROPY(ResultBSDF))
	{
		SubstrateRequestSharedLocalBasisTangent(OutSharedLocalBasisTypes, NewNormalIndex);
	}

#if SUBSTRATE_INLINE_SHADING
	// At this stage we do not know if this Substrate BSDF is going to be using simple volume lighting so always store the MFP on the side to not override SSSPROFILE data.
	ResultBSDF.TmpMFP =					lerp(BackgroundBSDF.TmpMFP,						ForegroundBSDF.TmpMFP,						ForegroundOtherMixFactor);

	// We blend the parameter above, and we also need to lerp the Coverage to match the total energy.
	ResultBSDF.Coverage =				lerp(BackgroundBSDF.Coverage,					ForegroundBSDF.Coverage,					HorizontalMixValue);
#endif

#if USE_DEVELOPMENT_SHADERS
	Result.PreviewColor = lerp(Background.PreviewColor, Foreground.PreviewColor, HorizontalMixValue);
#endif

#undef ResultBSDF
#undef ForegroundBSDF
#undef BackgroundBSDF
	return Result;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

float AddParameterBlendingBSDFCoverageToNormalMix(FSubstrateData ASubstrate, FSubstrateData BSubstrate)
{
	// This is a normalised mix so even though both BSDF have weights, the sum of weights used to combined them should always be 1.
#if SUBSTRATE_INLINE_SHADING
	const float SafeABSDFCoverage = saturate(ASubstrate.InlinedBSDF.Coverage);
	const float SafeBBSDFCoverage = saturate(BSubstrate.InlinedBSDF.Coverage);
	const float AMixFactor = SafeABSDFCoverage / max(SUBSTRATE_EPSILON, SafeABSDFCoverage + SafeBBSDFCoverage);
	return AMixFactor;
#else
	return 0.5f;
#endif
}

// ==> NOTE: Always pair with the compiler behavior in SubstrateCompilationInfoAddParamBlend
FSubstrateData SubstrateAddParameterBlending(FSubstrateData A, FSubstrateData B, float AMixFactor, uint NewNormalIndex, float ANoV, float BNoV)
{
	#if ALLOW_SUBSTRATE_TREE
	FSubstrateData Result = GetInitialisedSubstrateData();
#define ResultBSDF		Result.InlinedBSDF
#define ABSDF			A.InlinedBSDF
#define BBSDF			B.InlinedBSDF

	// We can only parameter blend the slab 
	BSDF_SETTYPE(ResultBSDF,			SUBSTRATE_BSDF_TYPE_SLAB);
	BSDF_SETSHAREDLOCALBASISID(ResultBSDF,NewNormalIndex);

	BSDF_SETISTOPLAYER(ResultBSDF,		0);
	BSDF_SETHASANISOTROPY(ResultBSDF,	BSDF_GETHASANISOTROPY(BBSDF)					| BSDF_GETHASANISOTROPY(ABSDF));
	BSDF_SETSSSTYPE(ResultBSDF,			SubstrateMergeSSSType(BSDF_GETSSSTYPE(BBSDF)		, BSDF_GETSSSTYPE(ABSDF)));
	BSDF_SETISTHIN(ResultBSDF,			BSDF_GETISTHIN(BBSDF)							| BSDF_GETISTHIN(ABSDF));
	BSDF_SETHASMFP(ResultBSDF,			BSDF_GETHASMFP(BBSDF)							| BSDF_GETHASMFP(ABSDF));
	BSDF_SETHASHAZINESS(ResultBSDF,		BSDF_GETHASHAZINESS(BBSDF)						| BSDF_GETHASHAZINESS(ABSDF));
	BSDF_SETHASF90(ResultBSDF,			BSDF_GETHASF90(BBSDF)							| BSDF_GETHASF90(ABSDF));
	BSDF_SETHASFUZZ(ResultBSDF,			BSDF_GETHASFUZZ(BBSDF)							| BSDF_GETHASFUZZ(ABSDF));

	// Bake the F90 into F0 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
	ABSDF.SubstrateBakeF90ToReflectivity(ANoV);
	BBSDF.SubstrateBakeF90ToReflectivity(BNoV);
	BSDF_SETHASF90(ResultBSDF, 0);

	// Bake the Specular Profiles into F0/F90 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
#if SUBSTRATE_SPECPROFILE_ENABLED
	ABSDF.SubstrateBakeSpecularProfileToReflectivity(ANoV);
	BBSDF.SubstrateBakeSpecularProfileToReflectivity(BNoV);
	BSDF_SETHASSPECPROFILE(ResultBSDF, 0);
#endif

#if SUBSTRATE_GLINTS_ENABLED 
	ABSDF.SubstrateBakeGlintToReflectivity();
	BBSDF.SubstrateBakeGlintToReflectivity();
	BSDF_SETHASGLINT(ResultBSDF, 0);
#endif // SUBSTRATE_GLINTS_ENABLED

	// Some parameters will contribute to added luminance: in this case we add them together and saturate to not go out of the safe range
	BSDF_SETEMISSIVE(ResultBSDF,		BSDF_GETEMISSIVE(BBSDF)								+ BSDF_GETEMISSIVE(ABSDF));
	SLAB_DIFFUSEALBEDO(ResultBSDF) =	saturate(SLAB_DIFFUSEALBEDO(BBSDF)					+ SLAB_DIFFUSEALBEDO(ABSDF));
	SLAB_F0(ResultBSDF) =				saturate(SLAB_F0(BBSDF)								+ SLAB_F0(ABSDF));
	SLAB_F90(ResultBSDF) =				saturate(SLAB_F90(BBSDF)							+ SLAB_F90(ABSDF));
	SLAB_FUZZ_COLOR(ResultBSDF) =		saturate(SLAB_FUZZ_COLOR(BBSDF)						+ SLAB_FUZZ_COLOR(ABSDF));

	// Some parameters are not contributing to added luminance: in this case we simply lerp them
	BSDF_SETTHICKNESSCM(ResultBSDF,		lerp(BSDF_GETTHICKNESSCM(BBSDF),				BSDF_GETTHICKNESSCM(ABSDF),					AMixFactor));
	SLAB_ROUGHNESS(ResultBSDF) =		lerp(SLAB_ROUGHNESS(BBSDF),						SLAB_ROUGHNESS(ABSDF),						AMixFactor);
	SLAB_ANISOTROPY(ResultBSDF) =		lerp(SLAB_ANISOTROPY(BBSDF),					SLAB_ANISOTROPY(ABSDF),						AMixFactor);
	SLAB_FUZZ_AMOUNT(ResultBSDF) =		lerp(SLAB_FUZZ_AMOUNT(BBSDF),					SLAB_FUZZ_AMOUNT(ABSDF),					AMixFactor);
	SLAB_FUZZ_ROUGHNESS(ResultBSDF) =	lerp(SLAB_FUZZ_ROUGHNESS(BBSDF),				SLAB_FUZZ_ROUGHNESS(ABSDF),					AMixFactor);
	SLAB_HAZINESS(ResultBSDF) =			LerpHazinessParameterBlending(BBSDF,			ABSDF,										AMixFactor);

	if (BSDF_GETSSSTYPE(ResultBSDF) == SSS_TYPE_DIFFUSION_PROFILE)
	{
		// As mentioned above, we cannot blend SSSProfiles. We do select the appropriate profile, ABSDF being selected if both have SSSProfile.
		if (BSDF_GETSSSTYPE(ABSDF) == SSS_TYPE_DIFFUSION_PROFILE)
		{
			SLAB_SSSPROFILEID(ResultBSDF) = SLAB_SSSPROFILEID(ABSDF);
			SLAB_SSSPROFILERADIUSSCALE(ResultBSDF) = SLAB_SSSPROFILERADIUSSCALE(ABSDF);
			SLAB_SSSPROFILETHICKNESSCM(ResultBSDF) = BSDF_GETTHICKNESSCM(ABSDF);
		}
		else
		{
			SLAB_SSSPROFILEID(ResultBSDF) = SLAB_SSSPROFILEID(BBSDF);
			SLAB_SSSPROFILERADIUSSCALE(ResultBSDF) = SLAB_SSSPROFILERADIUSSCALE(BBSDF);
		}
	}
	else
	{
		SLAB_SSSMFP(ResultBSDF) =		lerp(SLAB_SSSMFP(BBSDF),						SLAB_SSSMFP(ABSDF),						AMixFactor);
	}


#if SUBSTRATE_INLINE_SHADING
	// At this stage we do not know if this Substrate BSDF is going to be using simple volume lighting so always store the MFP on the side to not override SSSPROFILE data.
	ResultBSDF.TmpMFP =				lerp(BBSDF.TmpMFP,								ABSDF.TmpMFP,								AMixFactor);

	// When it comes to adding the visual contribution, we chose to add the coverage without clamping as currently done in the SubstrateAdd function.
	// Operations using Coverage are currently making sure they are in a safe range.
	ResultBSDF.Coverage =				saturate(BBSDF.Coverage + ABSDF.Coverage);
#endif

#if USE_DEVELOPMENT_SHADERS
	Result.PreviewColor = A.PreviewColor + B.PreviewColor;
#endif

#undef ResultBSDF
#undef ABSDF
#undef BBSDF
	return Result;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

float VerticalLayeringParameterBlendingBSDFCoverageToNormalMix(FSubstrateData TopSubstrate)
{
	// This is a normalised mix so even though both BSDF have weights, the sum of weights used to combined them should always be 1.
#if SUBSTRATE_INLINE_SHADING
	const float SafeTopBSDFCoverage = saturate(TopSubstrate.InlinedBSDF.Coverage);
	const float TopNormalContributionForWeightOfOne = 0.5f;
	// SUBSTRATE_TODO take into account transmittance when computing the normal weight
	return SafeTopBSDFCoverage * TopNormalContributionForWeightOfOne;
#else
	return 0.5f;
#endif
}

float SubstrateComputeTransmittanceNoV(float3 InF0, float3 InV, float3 InN, float InNoV)
{
	#if 1
	// For transmittance, we must account the fact that the view ray is going to be refracted at the interface of the slab.
	// Otherwise, at tangent view direction, the path length traveled within the medium will tend towards infinity. That result in too dark edges and is unrealistic.
	// To simplifiy things, we refract the view ray as if it was coming from a air interface into a medium of IOR deducted from F0.
	// If we do this computation once for each slab, it is equivalent to have each layered slab having the same IOR with a single view vector refraction at the top interface.
	// This is an aproximation we accept for the sake of performance today. 
	// OutRefractedTransmittanceNoV is used to build ThroughputV that will be used when processing the substrate tree to get correct throughput to layers below.
	// Ideas for later (requires an update of the substrate tree evaluation)
	//  1- we could compute IOR for each layers (gather IOR for each slab horizontally integrated) and refract V according to the from/to IOR a the interface.
	//  2- we could have V from a layer propagated to the next layer in order to have better specular match. 
	//  3- we could also store extra data in the closure in order to be able to account for that effect for the Lighting vector too.
	float3 RefractedV = refract(InV, -InN, 1.0 / DielectricF0ToIor(InF0.y));
	const float OutRefractedTransmittanceNoV = dot(RefractedV, InN);
	#else
	const float OutRefractedTransmittanceNoV = InNoV;
	#endif

	return OutRefractedTransmittanceNoV;
}

float SubstrateComputeTransmittanceNoV(float3 InF0, float InNoV)
{
	const float3 LocalN = float3(0, 0, 1);
	const float3 LocalV = float3(sqrtFast(1 - saturate(InNoV)), 0, InNoV);
	return SubstrateComputeTransmittanceNoV(InF0, LocalV, LocalN, InNoV);
}

FSubstrateData SubstrateVerticalLayeringParameterBlending(in FSubstrateData Top, in FSubstrateData Base, uint NewNormalIndex, float TopNoV, float BaseNoV)
{
	#if ALLOW_SUBSTRATE_TREE
	FSubstrateData Result = (FSubstrateData)0;
#define ResultBSDF		Result.InlinedBSDF
#define TopBSDF			Top.InlinedBSDF
#define BaseBSDF		Base.InlinedBSDF

#if SUBSTRATE_INLINE_SHADING
	const float SafeTopCoverage = saturate(TopBSDF.Coverage);
	const float SafeBaseCoverage = saturate(BaseBSDF.Coverage);
#else
	const float SafeTopCoverage = 0.5f;
	const float SafeBaseCoverage = 0.5f;
#endif

	// We can only parameter blend the slab 
	BSDF_SETTYPE(ResultBSDF,			SUBSTRATE_BSDF_TYPE_SLAB);
	BSDF_SETSHAREDLOCALBASISID(ResultBSDF,NewNormalIndex);

	BSDF_SETISTOPLAYER(ResultBSDF,		0);
	BSDF_SETHASANISOTROPY(ResultBSDF,	0);																					// Anisotropy is disabled because it changes a lot of things
	BSDF_SETSSSTYPE(ResultBSDF,			min(BSDF_GETSSSTYPE(BaseBSDF), SSS_TYPE_DIFFUSION));								// We only keep SSS if the bottom layer has it. Problem: cannot blend SSS profiles so we simply disable SSSProfile in this case, i.e., min(..., SSS_TYPE_DIFFUSION)
	BSDF_SETISTHIN(ResultBSDF,			BSDF_GETISTHIN(BaseBSDF));
	BSDF_SETHASMFP(ResultBSDF,			BSDF_GETHASMFP(BaseBSDF)						| BSDF_GETHASMFP(TopBSDF));			// We keep MFP of both to make sure bottom layer on translucent can also be effective
	BSDF_SETHASHAZINESS(ResultBSDF,		BSDF_GETHASHAZINESS(TopBSDF));														// We only keep haziness if the top layer has it
	BSDF_SETHASF90(ResultBSDF,			BSDF_GETHASF90(BaseBSDF)						| BSDF_GETHASF90(TopBSDF));			// F90: keep union of both even though it will be hard to get a match
	BSDF_SETHASFUZZ(ResultBSDF,			BSDF_GETHASFUZZ(BaseBSDF)						| BSDF_GETHASFUZZ(TopBSDF));		// Fuzz: keep union of both even though it will be hard to get a match

	// Bake the F90 into F0 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
	TopBSDF.SubstrateBakeF90ToReflectivity(TopNoV);
	BaseBSDF.SubstrateBakeF90ToReflectivity(BaseNoV);
	BSDF_SETHASF90(ResultBSDF, 0);

	// Bake the Specular Profiles into F0/F90 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
#if SUBSTRATE_SPECPROFILE_ENABLED
	TopBSDF.SubstrateBakeSpecularProfileToReflectivity(TopNoV);
	BaseBSDF.SubstrateBakeSpecularProfileToReflectivity(BaseNoV);
	BSDF_SETHASSPECPROFILE(ResultBSDF, 0);
#endif

#if SUBSTRATE_GLINTS_ENABLED 
	TopBSDF.SubstrateBakeGlintToReflectivity();
	BaseBSDF.SubstrateBakeGlintToReflectivity();
	BSDF_SETHASGLINT(ResultBSDF, 0);
#endif // SUBSTRATE_GLINTS_ENABLED

	// Saved original Base BBSDF MFP to not be changed be changed by EnableSlabBSDFSimpleVolumetric used to compute medium transmittance and scattering.
	// EnableSlabBSDFSimpleVolumetric will be called at then end of the SubstrateTree as needed later.
	const float3 OriginalBaseMFP = SLAB_SSSMFP(BaseBSDF);

	// Compute refracted V for evaluating transmittance attenuation used by the bottom layer.
	const float RefractedTransmittanceNoV = SubstrateComputeTransmittanceNoV(SLAB_F0(TopBSDF), TopNoV);

	// Compute the top slab transmittance that will be used to blend the top and base material parameters.
	// We do not check BSDF_GETISSIMPLEVOLUME(BSDFContext.BSDF) because that is set when we know the BSDF is not at the bottom later. And here we already know it is not the case.
	TopBSDF.EnableSlabBSDFSimpleVolumetric();
	BaseBSDF.EnableSlabBSDFSimpleVolumetric();
	FParticipatingMedia PM				= SubstrateSlabCreateParticipatingMedia(SLAB_DIFFUSEALBEDO(TopBSDF), SLAB_SSSMFP(TopBSDF));
	const float3 SlabTransmittanceV		= IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, RefractedTransmittanceNoV);
	const float3 SlabTransmittanceN		= IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, 1.0f /*Normal incidence i.e., NoL == 1.f*/);
	const float3 SlabTransmittance		= SlabTransmittanceV * SlabTransmittanceN;

	// We need to convert the scattering from the top layer to an additional albedo without light source direction.
	// There is no right answer here since we are adding energy without accounting for the directionality of directional or other local lights.
	// IsotropicMediumSlabEnvDirectionalAlbedo seems to set too much energy so instead the one for punctual light is used instead and looks more reasonable after some empirical tests.
	float3 TopSlabAlbedo = IsotropicMediumSlabPunctualDirectionalAlbedo(PM, TopNoV, 1.0f);

	// SUBSTRATE_TODO account for energy conservation here.

	FVerticalLayeringInfo Info			= GetVerticalLayeringInfo(SafeTopCoverage, SafeBaseCoverage);
	// Compute some ways to interpolate and accumulate parameters, but always normalise because coverage is applied later in UpdateSingleBSDFOperatorCoverageTransmittance.
	const float  TopToBottomLerp		= saturate((Info.TransmittanceTopAndBottom * dot(SlabTransmittance,0.33.xxx) + Info.SurfaceBottom) / max(SUBSTRATE_EPSILON, Info.Coverage));
	const float3 BottomColorFactor		= (Info.TransmittanceTopAndBottom * SlabTransmittance + Info.SurfaceBottom) / max(SUBSTRATE_EPSILON, Info.Coverage);
	const float  TopFactor				= Info.SurfaceTop / max(SUBSTRATE_EPSILON, Info.Coverage);
	
	// Add both layers thickness
	BSDF_SETTHICKNESSCM(ResultBSDF,		BSDF_GETTHICKNESSCM(BaseBSDF)												+ SafeTopCoverage * BSDF_GETTHICKNESSCM(TopBSDF));

	// Now add bottom layer component weights by throughput to the top layer.
	// Those parameters will contribute to added luminance, in this case we add them together and saturate to not go out of the safe range
	// Some parameters are not contributing to added luminance: in this case we simply lerp them
	BSDF_SETEMISSIVE(ResultBSDF,		BottomColorFactor * BSDF_GETEMISSIVE(BaseBSDF)								+ TopFactor * BSDF_GETEMISSIVE(TopBSDF));

	SLAB_DIFFUSEALBEDO(ResultBSDF) =	saturate(BottomColorFactor * SLAB_DIFFUSEALBEDO(BaseBSDF)					+ TopFactor * TopSlabAlbedo);
	SLAB_F0(ResultBSDF) =				saturate(BottomColorFactor * SLAB_F0(BaseBSDF)								+ TopFactor * SLAB_F0(TopBSDF));
	SLAB_F90(ResultBSDF) =				saturate(BottomColorFactor * SLAB_F90(BaseBSDF)								+ TopFactor * SLAB_F90(TopBSDF));
	SLAB_FUZZ_COLOR(ResultBSDF) =		saturate(BottomColorFactor * SLAB_FUZZ_COLOR(BaseBSDF)						+ TopFactor * SLAB_FUZZ_COLOR(TopBSDF));
		
	SLAB_ROUGHNESS(ResultBSDF) =		lerp(SLAB_ROUGHNESS(TopBSDF),					SLAB_ROUGHNESS(BaseBSDF),	TopToBottomLerp);
	SLAB_FUZZ_AMOUNT(ResultBSDF) =		lerp(SLAB_FUZZ_AMOUNT(TopBSDF),					SLAB_FUZZ_AMOUNT(BaseBSDF), TopToBottomLerp);
	SLAB_FUZZ_ROUGHNESS(ResultBSDF) =	lerp(SLAB_FUZZ_ROUGHNESS(TopBSDF),				SLAB_FUZZ_ROUGHNESS(BaseBSDF),TopToBottomLerp);
	SLAB_HAZINESS(ResultBSDF) =			LerpHazinessParameterBlending(TopBSDF,			BaseBSDF,					TopToBottomLerp);

	// Anisotropy is disabled with vertical blending. 
	SLAB_ANISOTROPY(ResultBSDF) =		0.f;

	// Keep the MFP form the bottom layer only and reduce it according to visibility. 
	// The top layer simple volume coat scattering/transmittance is baked with parameter blending.
	// Only the base is kept because the top layer MFP is usually large to keep the coat clear and this would result is wide scattering radius the SSS post process cannot resolve.
	SLAB_SSSMFP(ResultBSDF) =			OriginalBaseMFP * TopToBottomLerp;

#if SUBSTRATE_INLINE_SHADING
#if SUBSTRATE_OPAQUE_MATERIAL
	// Only the bottom layer SSS / Thin model is used so we only keep the bottom layer data there. Top layer is converted to throughput.
	ResultBSDF.TmpMFP =					BaseBSDF.TmpMFP * TopToBottomLerp;
	ResultBSDF.Coverage =				Info.Coverage;
#else

	// We compute the total transmittance of the medium over the max coverage
	const float MaxCoverage = max(0.000001, max(TopBSDF.Coverage, BaseBSDF.Coverage));
	FVerticalLayeringInfo Info2 = GetVerticalLayeringInfo(TopBSDF.Coverage / MaxCoverage, BaseBSDF.Coverage / MaxCoverage);
#if 0
	// Using transmittance as a way to interpolate should be better but there are issues with large values
	const float3 TopExtinction			= 1.0f / max(0.000001, TopBSDF.TmpMFP);
	const float3 BaseExtinction			= 1.0f / max(0.000001, BaseBSDF.TmpMFP);
	const float3 TopTransmitance		= exp(-TopExtinction);	// extinction = optical depth, we assume here d = 1 meter
	const float3 BaseTransmitance		= exp(-BaseExtinction);
	const float3 NewTransmittance		= Info2.TransmittanceOnlyBottom * BaseTransmitance + Info2.TransmittanceOnlyTop * TopTransmitance + Info2.TransmittanceTopAndBottom * TopTransmitance * BaseTransmitance;
	const float3 NewExtinction			= -log(saturate(max(0.0, NewTransmittance)));
	const float3 NewMFP					= 1.0f / NewExtinction;
#else
	// MFP looks more perceptually linear even though it is not perfect
	const float3 NewMFP					= Info2.TransmittanceOnlyBottom* BaseBSDF.TmpMFP + Info2.TransmittanceOnlyTop * TopBSDF.TmpMFP + Info2.TransmittanceTopAndBottom * min(TopBSDF.TmpMFP, BaseBSDF.TmpMFP);
#endif

	ResultBSDF.Coverage					= MaxCoverage;
	ResultBSDF.TmpMFP					= NewMFP;
	ResultBSDF.EnableSlabBSDFSimpleVolumetric();	// Derive Slab MFP from TmpMFP and thickness. Due to approximation above (thickness & MFP based combination), it is not a perfect match.

#endif

#endif

#if USE_DEVELOPMENT_SHADERS
	Result.PreviewColor = lerp(Top.PreviewColor, Base.PreviewColor, 0.5);
#endif

#undef ResultBSDF
#undef TopBSDF
#undef BaseBSDF
	return Result;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

FSubstrateData SubstrateWeightParameterBlending(FSubstrateData A, float Weight)
{
	#if ALLOW_SUBSTRATE_TREE
	const float SafeWeight = saturate(Weight);
	FSubstrateData SubstrateData = A;

	// Apply the weight to coverage
#if SUBSTRATE_INLINE_SHADING
	SubstrateData.InlinedBSDF.Coverage *= SafeWeight;
#endif

#if USE_DEVELOPMENT_SHADERS
	SubstrateData.PreviewColor *= SafeWeight;
#endif

	return SubstrateData;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

float SelectParameterBlendingToNormal(float SelectValue)
{
	if (SelectValue <= 0.0f)
	{
		return 0.0f;
	}
	return 1.0f;
}

FSubstrateData SubstrateSelectParameterBlending(FSubstrateData A, FSubstrateData B, float SelectValue, uint NewNormalIndex, inout uint OutSharedLocalBasisTypes)
{
	#if ALLOW_SUBSTRATE_TREE
	// It is fine to reuse A&B and morph them since they are here a copy of the original SubstrateData.
	if (SelectValue <= 0.0f)
	{
		BSDF_SETSHAREDLOCALBASISID(A.InlinedBSDF, NewNormalIndex);
		if (!SUBSTRATE_FASTPATH && BSDF_GETHASANISOTROPY(A.InlinedBSDF))
		{
			SubstrateRequestSharedLocalBasisTangent(OutSharedLocalBasisTypes, NewNormalIndex);
		}
		return A;
	}

	BSDF_SETSHAREDLOCALBASISID(B.InlinedBSDF, NewNormalIndex);
	if (!SUBSTRATE_FASTPATH && BSDF_GETHASANISOTROPY(B.InlinedBSDF))
	{
		SubstrateRequestSharedLocalBasisTangent(OutSharedLocalBasisTypes, NewNormalIndex);
	}
	return B;
	#else
	return GetInitialisedSubstrateData();
	#endif
}

void SubstrateGetThinFilmF0F90(float NoV, float ThinFilmNormalizedThickness, float ThinFilmIOR, inout float3 OutF0, inout float3 OutF90)
{
	const float3 F0 = OutF0;
	const float3 F90 = OutF90;

	const float NoL = NoV;
	const float VoH = NoV;
	float3 RThinFilm = F_ThinFilm(NoV, NoL, VoH, F0, F90, ThinFilmIOR, ThinFilmNormalizedThickness);

	// Compute a F0 which match evaluation Thin-Film reflectance for the current view angle using Schlick's Fresnel
	const float FReference = min(0.99f, F_Schlick(0, 1.0f, NoV).x);
	RThinFilm = max(RThinFilm, FReference);

	float3 F0_Thin = (RThinFilm - F90 * FReference) / (1 - FReference);

	// Ensure the generated F0 doesn't cause micro-oclusion
	F0_Thin = max(GetF0MicroOcclusionThreshold(), F0_Thin);

	// If micro-occlusion was set, re-apply it.		
	const float3 MicroOcclusion = F0RGBToMicroOcclusion(F0);
	F0_Thin *= MicroOcclusion;

	OutF0 = F0_Thin;
	OutF90 = max(F0_Thin, F90);
}

struct FSubstrateThinFilmOutput
{
	float3 F0;
	float3 F90;
};
FSubstrateThinFilmOutput SubstrateGetThinFilmF0F90(float NoV, float3 F0, float3 F90, float ThinFilmNormalizedThickness, float ThinFilmIOR)
{
	FSubstrateThinFilmOutput Output;
	Output.F0 = F0;
	Output.F90 = F90;
	SubstrateGetThinFilmF0F90(NoV, ThinFilmNormalizedThickness, ThinFilmIOR, Output.F0, Output.F90);
	return Output;
}

///////////////////////////////////////////////////////////////////////////////
// Functions used to sanitize BSDF before they are used for lighting (when forward) or stored to memory (base pass).
// We do not normalize normals, this is done before the lighting step.
// This step is obligatory so we apply View.MinRoughness here for forward and deferred.

float SanitizeRoughness(float Roughness)
{
#if MATERIAL_FULLY_ROUGH
	return 1.0f;
#else
	#if (USE_EDITOR_SHADERS && !ES3_1_PROFILE) || MOBILE_EMULATION
	// This feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	// If more view members are accessed, make sure they are patched into the SharedLumenView from FDeferredShadingSceneRenderer::UpdateLumenScene.
	Roughness = Roughness * ResolvedView.RoughnessOverrideParameter.y + ResolvedView.RoughnessOverrideParameter.x;
	#endif
	return saturate(max(View.MinRoughness, Roughness));
#endif
}

float3 SanitizeDiffuseAlbedo(float3 DiffuseAlbedo)
{
	#if USE_DEVELOPMENT_SHADERS
	// This feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	// If more view members are accessed, make sure they are patched into the SharedLumenView from FDeferredShadingSceneRenderer::UpdateLumenScene.
	return saturate(DiffuseAlbedo) * ResolvedView.DiffuseOverrideParameter.www + ResolvedView.DiffuseOverrideParameter.xyz;
	#else
	return saturate(DiffuseAlbedo);
	#endif
}

float3 SanitizeF0(float3 F0)
{
	#if USE_DEVELOPMENT_SHADERS
	// This feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	// If more view members are accessed, make sure they are patched into the SharedLumenView from FDeferredShadingSceneRenderer::UpdateLumenScene.
	return saturate(F0) * ResolvedView.SpecularOverrideParameter.w + ResolvedView.SpecularOverrideParameter.xyz;
	#else
	return saturate(F0);
	#endif
}

float SanitizeF0(float F0)
{
	#if USE_DEVELOPMENT_SHADERS
	// This feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
	// If more view members are accessed, make sure they are patched into the SharedLumenView from FDeferredShadingSceneRenderer::UpdateLumenScene.
	return saturate(F0) * ResolvedView.SpecularOverrideParameter.w + ResolvedView.SpecularOverrideParameter.x;
	#else
	return saturate(F0);
	#endif
}

float3 SanitizeSSSMFP(float3 InMFP)
{
	return clamp(InMFP, 0.0f, Max111110BitsFloat3);
}

float SanitizeAnisotropy(float Anisotropy)
{
	return clamp(Anisotropy, -1.f, 1.f);
}

#define SUBSTRATE_IS_VALID_DIRECTION(Direction) (!all(Direction == 0) && all(IsFinite(Direction)))

void SanitizeAndNormalizeSharedLocalBases(inout FSharedLocalBases SharedLocalBases, in float3x3 TangentToWorld)
{
	// We must sanitize each normal to make sure nans are not stored which can expanlater to fire flies in the image.
	// We must normalize each normal and tangent to avoid non normalised vectors due to per vertex interpolation or texture filtering,
	// for the deferred (our oct packing relies on normalized normal) and forward (normals are going to be used as-is from registers) paths.
	uint i = 0;
#if SUBSTRATE_CLAMPED_CLOSURE_COUNT == 1
	if(SharedLocalBases.Count > 0) // Equivalent to Count==1
#else
	UNROLL
	for (; i < SharedLocalBases.Count; ++i)
#endif
	{
		const float3 InputNormal = SharedLocalBases.Normals[i];
		if(!SUBSTRATE_IS_VALID_DIRECTION(InputNormal))
		{
			const float3 FallbackNormal = TangentToWorld[2];
		#if IS_NANITE_PASS // With nanite we do more check in case the TangentToWorld is wrong. (noticed in some material but we have not been able to catch the bug yet)
			SharedLocalBases.Normals[i] = SUBSTRATE_IS_VALID_DIRECTION(FallbackNormal) ? FallbackNormal : float3(0, 0, 1);
		#else
			SharedLocalBases.Normals[i] = FallbackNormal;
		#endif
		}
		SharedLocalBases.Normals[i] = normalize(SharedLocalBases.Normals[i]);

		if (SubstrateGetSharedLocalBasisType(SharedLocalBases.Types, i) == SUBSTRATE_BASIS_TYPE_TANGENT)
		{
			const float3 InputTangent = SharedLocalBases.Tangents[i];
			if (!SUBSTRATE_IS_VALID_DIRECTION(InputTangent))
			{
				const float3 FallbackTangent = TangentToWorld[0];
			#if IS_NANITE_PASS
				SharedLocalBases.Tangents[i] = SUBSTRATE_IS_VALID_DIRECTION(FallbackTangent) ? FallbackTangent : float3(1, 0, 0);
			#else
				SharedLocalBases.Tangents[i] = FallbackTangent;
			#endif
			}
			SharedLocalBases.Tangents[i] = normalize(SharedLocalBases.Tangents[i]);
		}
	}
}

void FSubstrateBSDF::SubstrateSanitizeBSDF()
{
	switch (BSDF_GETTYPE(this))
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));

			SLAB_DIFFUSEALBEDO(this) = SanitizeDiffuseAlbedo(SLAB_DIFFUSEALBEDO(this));
			SLAB_F0(this) = SanitizeF0(SLAB_F0(this));
			SLAB_F90(this) = saturate(SLAB_F90(this));
			SLAB_ROUGHNESS(this) = SanitizeRoughness(SLAB_ROUGHNESS(this));
			if (BSDF_GETHASHAZINESS(this))
			{
				FHaziness LocalHaziness = UnpackHaziness(SLAB_HAZINESS(this));
				LocalHaziness.Roughness = SanitizeRoughness(LocalHaziness.Roughness);
				SLAB_HAZINESS(this) = PackHaziness(LocalHaziness);
			}
			SLAB_ANISOTROPY(this) = SanitizeAnisotropy(SLAB_ANISOTROPY(this));
			// SLAB_HAZINESS is sanitize when packed, see PackHaziness.
		#if SUBSTRATE_INLINE_SHADING && SUBSTRATE_OPAQUE_MATERIAL
			// We only clamp MFP when we are going to export it packed into the GBuffer.
			// Otherwise, for forward shading, it is already safe to use, see RescaleMFPToComputationSpace.
			TmpMFP = clamp(TmpMFP, 0.0f, Max111110BitsFloat3);
		#endif

			// BSDF feature tests must be executed for profile path to not interact with MFP, especially for forward rendered simple volumetrics.
			if (BSDF_GETSSSTYPE(this) == SSS_TYPE_DIFFUSION_PROFILE)
			{
				SLAB_SSSPROFILERADIUSSCALE(this) = saturate(SLAB_SSSPROFILERADIUSSCALE(this));
			}
			else if (BSDF_GETSSSTYPE(this) == SSS_TYPE_DIFFUSION)
			{
				SLAB_SSSMFP(this) = SanitizeSSSMFP(SLAB_SSSMFP(this));
			}

			SLAB_FUZZ_AMOUNT(this) = saturate(SLAB_FUZZ_AMOUNT(this));
			SLAB_FUZZ_COLOR(this) = saturate(SLAB_FUZZ_COLOR(this));
			SLAB_FUZZ_ROUGHNESS(this) = saturate(SLAB_FUZZ_ROUGHNESS(this));
			break; 
		}
		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));

			HAIR_BASECOLOR(this) = SanitizeDiffuseAlbedo(HAIR_BASECOLOR(this));
			HAIR_SCATTER(this) = saturate(HAIR_SCATTER(this));
			HAIR_ROUGHNESS(this) = SanitizeRoughness(HAIR_ROUGHNESS(this));
			HAIR_SPECULAR(this) = SanitizeF0(HAIR_SPECULAR(this));
			HAIR_BACKLIT(this) = saturate(HAIR_BACKLIT(this));
			break;
		}
		case SUBSTRATE_BSDF_TYPE_EYE:
		{
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));

			EYE_DIFFUSEALBEDO(this) = SanitizeDiffuseAlbedo(EYE_DIFFUSEALBEDO(this));
			EYE_ROUGHNESS(this) = SanitizeRoughness(EYE_ROUGHNESS(this));
			EYE_IRISDISTANCE(this) = saturate(EYE_IRISDISTANCE(this));
			EYE_IRISMASK(this) = saturate(EYE_IRISMASK(this));
			EYE_IRISNORMAL(this) = normalize(EYE_IRISNORMAL(this));
			EYE_IRISPLANENORMAL(this) = normalize(EYE_IRISPLANENORMAL(this));
			break;
		}
		case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		{
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));

			SLW_BASECOLOR(this) = SanitizeDiffuseAlbedo(SLW_BASECOLOR(this));
			SLW_METALLIC(this) = saturate(SLW_METALLIC(this));
			SLW_SPECULAR(this) = SanitizeF0(SLW_SPECULAR(this));
			SLW_ROUGHNESS(this) = SanitizeRoughness(SLW_ROUGHNESS(this));
			SLW_TOPMATERIALOPACITY(this) = saturate(SLW_TOPMATERIALOPACITY(this));
		#if SUBSTRATE_INLINE_SINGLELAYERWATER
			SLW_WATERALBEDO(this) = saturate(SLW_WATERALBEDO(this));
			SLW_WATEREXTINCTION(this) = clamp(SLW_WATEREXTINCTION(this), 0.0f, Max10BitsFloat);
			SLW_WATERPHASEG(this) = clamp(SLW_WATERPHASEG(this), -1.0f, 1.0f);
			SLW_COLORSCALEBEHINDWATER(this) = clamp(SLW_COLORSCALEBEHINDWATER(this), 0.0f, Max10BitsFloat);
		#endif
			break;
		}
		case SUBSTRATE_BSDF_TYPE_UNLIT:
		{
		#if MATERIAL_ALLOW_NEGATIVE_EMISSIVECOLOR == 0 // Only Unlit can have negative emissive if specified
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));
		#endif
			UNLIT_TRANSMITTANCE(this) = saturate(UNLIT_TRANSMITTANCE(this));
			UNLIT_NORMAL(this) = normalize(UNLIT_NORMAL(this));
			break;
		}
		case SUBSTRATE_BSDF_TYPE_TOON:
		{
			// We do not clamp emissive luminance upper bound. This is done later in the base pass within pre-exposed luminance space.
			BSDF_SETEMISSIVE(this, max(BSDF_GETEMISSIVE(this), 0.0));

			TOON_BASECOLOR(this) = SanitizeDiffuseAlbedo(TOON_BASECOLOR(this));
			TOON_METALLIC(this) = saturate(TOON_METALLIC(this));
			TOON_SPECULAR(this) = saturate(TOON_SPECULAR(this));
			TOON_ROUGHNESS(this) = clamp(TOON_ROUGHNESS(this), 0.0, 1.0);	// We do not use SanitizeRoughness which clamps to View.MinRoughness that is higher than 0.
			TOON_PATTERNUVS(this) = clamp(TOON_PATTERNUVS(this), 0.0.xx, 1.0.xx);
			TOON_ANISOTROPY(this) = clamp(TOON_ANISOTROPY(this), -1.0, 1.0);
			break;
		}
	}
}

// Convert to simple volume to get some transmittance
void FSubstrateBSDF::PostProcessSlabBeforeLighting(bool bIsSubstrateOpaqueMaterial)
{
	// Force simple volume and MFP rescaling if the layer is not at the bottom or if sss type is already SimpleVolume. 
	// In this case, it only rescales the MFP. The MFP rescaling can't be done before, as the thickness needs to be 
	// preserved (i.e. not normalized) for rough refraction computation
	// Side-effect:
	// * Rescale MFP to computation space
	// * Set thickness to normalized simple volume thickness
	// * Set SSS type to SimpleVolume
	if (BSDF_GETHASMFP(this))
	{
#if PATH_TRACING
		// In the case of Path Tracing, we implement the support of MFP on the bottom most slab of solid surfaces with a real random walk
		// against the geometry, not the implied thickness layer.  Therefore in this case we don't want to modify the value provided in the
		// material graph.
		if (!this.bIsBottom || (BSDF_GETSSSTYPE(this) == SSS_TYPE_SIMPLEVOLUME && BSDF_GETISTHIN(this)))
#else
		if (!this.bIsBottom || BSDF_GETSSSTYPE(this) == SSS_TYPE_SIMPLEVOLUME)
#endif
		{
			this.EnableSlabBSDFSimpleVolumetric();
		}
	}
}

void FSubstrateBSDF::PostProcessBSDFBeforeLighting(bool bIsSubstrateOpaqueMaterial)
{
	switch (BSDF_GETTYPE(this))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:
	{
		this.PostProcessSlabBeforeLighting(bIsSubstrateOpaqueMaterial);
		break;
	}
	}
}

bool FSubstrateBSDF::HasScatteringData()
{
	const uint SSSType = BSDF_GETSSSTYPE(this);
	return SSSType != SSS_TYPE_NONE && SSSType != SSS_TYPE_SIMPLEVOLUME;
}

bool FSubstrateBSDF::HasBackScattering()
{
	const uint SSSType = BSDF_GETSSSTYPE(this);
	return SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP;
}

bool FSubstrateBSDF::HasSSSDiffusion()
{
	const uint SSSType = BSDF_GETSSSTYPE(this);
	return SSSType == SSS_TYPE_DIFFUSION || SSSType == SSS_TYPE_DIFFUSION_PROFILE;
}

void FSubstrateBSDF::SubstrateBakeF90ToReflectivity(in float NoV)
{
	if (BSDF_GETHASF90(this))
	{
		float3 F90 = saturate(SLAB_F90(this));
		// What is important is to maintain the hue and saturation, so we scale the color by the maximum of its components
		const float Divisor = max3(F90.r, F90.g, F90.b);
		F90 = Divisor > 0.0f ? F90 * rcp(Divisor) : 1.0f;

		// Shift when F90 color is fully reached to better match reference
		float3 F0Scale = lerp(1.f, F90, Pow5((1-NoV)*1.2f));

		// Fade out F90 effect as roughness goes from 0 to 0.5. 
		// On Adaptive GBuffer, the effect is no longer visible after roughness 0.5.
		F0Scale = lerp(1.f, F0Scale, saturate(1.f-2.f*SLAB_ROUGHNESS(this)));		

		SLAB_F0(this) *= F0Scale;
		SLAB_F90(this) = 1.f;
		BSDF_SETHASSPECPROFILE(this, 0);
	}
}

void FSubstrateBSDF::SubstrateBakeSpecularProfileToReflectivity(in float NoV)
{
#if SUBSTRATE_SPECPROFILE_ENABLED
	if (BSDF_GETHASSPECPROFILE(this))
	{
		// Evaluate the spec lut
		const float SpecProfileID = SLAB_SPECPROFILEID(this);
		const float3 SpecLUTSample = EvaluateSpecularProfile(SpecProfileID, NoV, 1.0f/*NoL*/, 1.0f/*VoH*/, 1.0f/*NoH*/);
		SLAB_F0(this) *= SpecLUTSample;
		SLAB_F90(this) *= SpecLUTSample;
		BSDF_SETHASSPECPROFILE(this, 0);
	}
#endif
}

void FSubstrateBSDF::SubstrateBakeGlintToReflectivity()
{
#if SUBSTRATE_COMPLEXSPECIALPATH
	// It does not work to blend glints attributes when using parameter blending or fully simplified materials.
	// So for such case we bake the glints as a specular lobe, reducing reflectivity according to glint amount/coverage.
	if (BSDF_GETHASGLINT(this))
	{
		const float GlintCoverage = saturate(SLAB_GLINT_VALUE(this));
		SLAB_F0(this) *= GlintCoverage;
		SLAB_F90(this) *= GlintCoverage;
		BSDF_SETHASGLINT(this, 0);
	}
#endif
}

void FSubstrateBSDF::SubstrateBakeHazinessToSingleRoughness()
{
#if !SUBSTRATE_FASTPATH
	if (BSDF_GETHASHAZINESS(this))
	{
		FHaziness LocalHaziness = UnpackHaziness(SLAB_HAZINESS(this));

		// Only blend roughness when we are not in the simple clear coat mode, otherwise we want to keep the original roughness for the legacy clear coat shading model.
		if (!LocalHaziness.bSimpleClearCoat)
		{
			const float SlabRoughess = SLAB_ROUGHNESS(this);
			SLAB_ROUGHNESS(this) = lerp(SlabRoughess, LocalHaziness.Roughness, LocalHaziness.Weight);
			BSDF_SETHASHAZINESS(this, 0);

			SLAB_HAZINESS(this) = PackHaziness(LocalHaziness);
		}
	}
#endif
}

#if !SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
void UpdateSingleBSDFOperatorCoverageTransmittance(
	inout FSubstrateTree SubstrateTree,
	FSubstratePixelHeader SubstratePixelHeader,
	int BSDFIndex,
	FSubstrateIntegrationSettings Settings,
	float3 V);
#endif

///////////////////////////////////////////////////////////////////////////////
// Other tools

// Those participating media funtion are use by the Slab node. In this case, the MFP is considered as a simple MFP because MFP is not directly inversible to medium properties.

FParticipatingMedia SubstrateSlabCreateParticipatingMedia(float3 DiffuseColor, float3 MeanFreePathCentimeters)
{
	const float3 MeanFreePathMeters = MeanFreePathCentimeters * CENTIMETER_TO_METER;
	return CreateMediumFromBaseColorMFP(DiffuseColor, MeanFreePathMeters);
}

float SubstrateSlabDiffuseToVolumeBlend(FParticipatingMedia PM)
{
	// Remember, computation are made for a slab of 1 metter, according to SUBSTRATE_SIMPLEVOLUME_THICKNESS_M.
	// We start blending from diffuse toward a slab of participating media when the mean free path reaches 4 centimeters.
	const float MaxMeanFreePathMeters = max(PM.MeanFreePath.x, max(PM.MeanFreePath.y, PM.MeanFreePath.z));
	// Start blending volumetric after mean free path of 4 centimeters (also needed because threshold when recovering extinction from transmittance cause a small minimum value)
	const float StartVolumeBlenMFP = 0.04f;
	// And when the mean free path reaches 1/3.0f = 0.33 meter. That is extinction=1/0.33=3, transmittance at 1m (bottom of the slab) = exp(-3) = 0.05) we fully use the volumetric model.
	const float Blend = saturate(max(0.0, MaxMeanFreePathMeters - StartVolumeBlenMFP) * 3.0f);
	//const float Blend = max(0.0, MaxMeanFreePathMeters - StartVolumeBlenMFP) > 0.0 ? 1.0 : 0.0;

	// Using a non linear blend helps smoothing out transitions visually.
	return Blend * Blend;
}

float3 RescaleMFPToComputationSpace(float3 InMFPInCm, float InSrcThicknessInCm, float InDstThicknessInCm)
{
	// Rescale the BSDF mean free path so to run volumetric material math on a unit slab of medium whose thickness is defined by InDstThicknessInCm
	// For instance, if thickness is larger => we need to reduce the MFP for the medium to appear visually thicker.
	// We do not cap the recovered MFP to be able to get closer to and indentity transmittance=1.0. 
	// MFP is safe already thanks to SUBSTRATE_EPSILON and TransmittanceToMeanFreePath. And it will be made safe later with a SanitizeSubstrateSlab before it is exported to the gbuffer.
	return max(InMFPInCm * (InDstThicknessInCm / max(SUBSTRATE_EPSILON, InSrcThicknessInCm)), 0.0f);
}

void FSubstrateBSDF::EnableSlabMFPComputationSpace()
{
#if SUBSTRATE_INLINE_SHADING
	// Now we know that this is a simple volume so we can freely set the mean free path
	SLAB_SSSMFP(this) = this.TmpMFP;
#endif

	// Rescale the BSDF mean free path so that we can drop the thickness parameter and run volumetric material math on a unit slab of medium
	// For instance, if thickness is larger => we need to reduce the MFP for the medium to appear visually thicker.
	SLAB_SSSMFP(this) = RescaleMFPToComputationSpace(SLAB_SSSMFP(this), BSDF_GETTHICKNESSCM(this), SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM);
}

void FSubstrateBSDF::EnableSlabBSDFSimpleVolumetric()
{
	this.EnableSlabMFPComputationSpace();

	// Mark the BSDF has not having any SSS through post process.
	// Notify that the Slab can use SSS post-process only if at the bottom (layered considered optically thick, e.g. cannot see-through)
	// When forward rendering is used or if a layer is not a bottom layer, SSS post cannot be used and simple scattering is used.
	BSDF_SETSSSTYPE(this, SSS_TYPE_SIMPLEVOLUME);
}

///////////////////////////////////////////////////////////////////////////////
// Generic BSDF accessors

uint SubstrateGetBSDFType(in FSubstrateBSDF BSDF)
{
	return BSDF_GETTYPE(BSDF);
}

float3 SubstrateGetBSDFBaseColor(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return lerp(SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), F0RGBToMetallic(SLAB_F0(BSDF)));  // This BaseColor recovery based on Metallic recovers is not accurate, and only used for legacy conversion purpose (e.g., PostProcessMaterial texture access)
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return HAIR_BASECOLOR(BSDF);
	case SUBSTRATE_BSDF_TYPE_EYE: 				return EYE_DIFFUSEALBEDO(BSDF);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return SLW_BASECOLOR(BSDF);
	case SUBSTRATE_BSDF_TYPE_TOON:				return TOON_BASECOLOR(BSDF);
	}
	return 0.f;
}

float3 SubstrateGetBSDFDiffuseColor(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return SLAB_DIFFUSEALBEDO(BSDF);
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return HAIR_BASECOLOR(BSDF);
	case SUBSTRATE_BSDF_TYPE_EYE: 				return EYE_DIFFUSEALBEDO(BSDF);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return SLW_BASECOLOR(BSDF);
	case SUBSTRATE_BSDF_TYPE_TOON:				return TOON_BASECOLOR(BSDF);
	}
	return 0.f;
}

void SubstrateSetBSDFDiffuseColor(inout FSubstrateBSDF BSDF, float3 InColor)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				SLAB_DIFFUSEALBEDO(BSDF) = InColor; break;
	case SUBSTRATE_BSDF_TYPE_HAIR: 				HAIR_BASECOLOR(BSDF) = InColor; break;
	case SUBSTRATE_BSDF_TYPE_EYE: 				EYE_DIFFUSEALBEDO(BSDF) = InColor; break;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	SLW_BASECOLOR(BSDF) = InColor; break;
	case SUBSTRATE_BSDF_TYPE_TOON:				TOON_BASECOLOR(BSDF) = InColor; break;
	}
}

float3 SubstrateGetBSDFSpecularF0(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return SLAB_F0(BSDF);
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return DielectricSpecularToF0(HAIR_SPECULAR(BSDF));
	case SUBSTRATE_BSDF_TYPE_EYE: 				return EYE_F0(BSDF);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return DielectricSpecularToF0(SLW_SPECULAR(BSDF));
	case SUBSTRATE_BSDF_TYPE_TOON:				return DielectricSpecularToF0(TOON_SPECULAR(BSDF));
	}
	return 0.f;
}

float3 SubstrateGetBSDFSpecularF90(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return SLAB_F90(BSDF);
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return 1.0f;
	case SUBSTRATE_BSDF_TYPE_EYE: 				return EYE_F90(BSDF);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return 1.0f;
	}
	return 0.f;
}

void SubstrateSetBSDFSpecularF0(in FSubstrateBSDF BSDF, float3 F0)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				SLAB_F0(BSDF) = F0; break;
	case SUBSTRATE_BSDF_TYPE_HAIR: 				HAIR_SPECULAR(BSDF) = F0RGBToDielectricSpecular(F0); break;
	case SUBSTRATE_BSDF_TYPE_EYE: 				/* EYE_F0 us hardcoded */ break;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	SLW_SPECULAR(BSDF) = F0RGBToDielectricSpecular(F0); break;
	case SUBSTRATE_BSDF_TYPE_TOON:				TOON_SPECULAR(BSDF) = F0RGBToDielectricSpecular(F0); break;
	}
}

void SubstrateSetBSDFSpecularF90(in FSubstrateBSDF BSDF, float3 F90)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				SLAB_F90(BSDF) = F90; break;
	case SUBSTRATE_BSDF_TYPE_HAIR: 				break;
	case SUBSTRATE_BSDF_TYPE_EYE: 				break;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	break;
	}
}

float3 SubstrateGetBSDFSubSurfaceColor(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:
	{
		if (BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE)
		{
			const float3 Extinction = 1.0f / max(SUBSTRATE_EPSILON, SLAB_SSSMFP(BSDF) * CENTIMETER_TO_METER);
			return ExtinctionToTransmittance(Extinction, SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
		}
		else if (BSDF_GETHASFUZZ(BSDF))
		{
			return SLAB_FUZZ_COLOR(BSDF); // Match legacy subsurface color preview
		}
		return 0;
	}
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return 0;
	case SUBSTRATE_BSDF_TYPE_EYE: 				return EYE_DIFFUSEALBEDO(BSDF); // Eye only has SSS via SubsurfaceProfile, so simply give the diffuse.
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return 0;
	}
	return 0.f;
}

uint SubstrateGetBSDFSubSurfaceProfileID(in FSubstrateBSDF BSDF)
{
	if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
	{
		switch (BSDF_GETTYPE(BSDF))
		{
			case SUBSTRATE_BSDF_TYPE_SLAB:	return SLAB_SSSPROFILEID(BSDF);
			case SUBSTRATE_BSDF_TYPE_EYE:	return EYE_SSSPROFILEID(BSDF);
		}
	}
	return 0;
}

float SubstrateGetBSDFSpecular(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return F0RGBToDielectricSpecular(SLAB_F0(BSDF));
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return HAIR_SPECULAR(BSDF);
	case SUBSTRATE_BSDF_TYPE_EYE: 				return F0ToDielectricSpecular(EYE_F0(BSDF).x);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return SLW_SPECULAR(BSDF);
	case SUBSTRATE_BSDF_TYPE_TOON:				return TOON_SPECULAR(BSDF);
	}
	return 0.f;
}

float SubstrateGetBSDFMetallic(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB: 				return F0RGBToMetallic(SLAB_F0(BSDF)); // Metallic recovers is not accurate, and only used for legacy conversion purpose (e.g., PostProcessMaterial texture access)
	case SUBSTRATE_BSDF_TYPE_HAIR: 				return 0;
	case SUBSTRATE_BSDF_TYPE_EYE: 				return 0;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return 0;
	case SUBSTRATE_BSDF_TYPE_TOON:				return TOON_METALLIC(BSDF);
	}
	return 0.f;
}

float SubstrateGetBSDFRoughness(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:				return SLAB_ROUGHNESS(BSDF);
	case SUBSTRATE_BSDF_TYPE_HAIR:				return HAIR_ROUGHNESS(BSDF);
	case SUBSTRATE_BSDF_TYPE_EYE:				return EYE_ROUGHNESS(BSDF);
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return SLW_ROUGHNESS(BSDF);
	case SUBSTRATE_BSDF_TYPE_TOON:				return TOON_ROUGHNESS(BSDF);
	}
	return 0.f;
}

void FSubstrateBSDF::SubstrateSetBSDFRoughness(in float Roughness)
{
	switch (BSDF_GETTYPE(this))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:				SLAB_ROUGHNESS(this) = Roughness; break;
	case SUBSTRATE_BSDF_TYPE_HAIR:				HAIR_ROUGHNESS(this) = Roughness; break;
	case SUBSTRATE_BSDF_TYPE_EYE:				EYE_ROUGHNESS(this)  = Roughness; break;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	SLW_ROUGHNESS(this)  = Roughness; break;
	case SUBSTRATE_BSDF_TYPE_TOON:				TOON_ROUGHNESS(this)  = Roughness; break;
	}
}

float SubstrateGetBSDFAnisotropy(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:				return BSDF_GETHASANISOTROPY(BSDF) ? SLAB_ANISOTROPY(BSDF) : 0.f;
	case SUBSTRATE_BSDF_TYPE_HAIR:				return 0;
	case SUBSTRATE_BSDF_TYPE_EYE:				return 0;
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:	return 0;
	case SUBSTRATE_BSDF_TYPE_TOON:				return BSDF_GETHASANISOTROPY(BSDF) ? TOON_ANISOTROPY(BSDF) : 0.f;
	}
	return 0.f;
}

float SubstrateGetBSDFOpacity(in FSubstrateBSDF BSDF)
{
	// Recompute the 'legacy' opacity value based on feature/BSDF type
	float Out = 0.f;
	if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB)
	{
		if (BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE)
		{
			switch (BSDF_GETSSSTYPE(BSDF))
			{
				case SSS_TYPE_WRAP				: Out = 1.f-SLAB_SSSPHASEANISOTROPY(BSDF); break;
				case SSS_TYPE_TWO_SIDED_WRAP	: Out = 1.f-SLAB_SSSPHASEANISOTROPY(BSDF); break;
				case SSS_TYPE_DIFFUSION			: Out = 1.f; break;
				case SSS_TYPE_DIFFUSION_PROFILE	: Out = SLAB_SSSPROFILERADIUSSCALE(BSDF); break;
				case SSS_TYPE_SIMPLEVOLUME		: Out = 1.f; break;
			}
		}
		else if (BSDF_GETHASFUZZ(BSDF))
		{
			Out = SLAB_FUZZ_AMOUNT(BSDF);	// Cloth
		}
		else if (BSDF_GETHASHAZINESS(BSDF))
		{
			const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
			if (Haziness.bSimpleClearCoat)
			{
				Out = Haziness.Weight;		// ClearCoat
			}
		}
	}
	else if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_EYE)
	{
		Out = 1 - EYE_IRISMASK(BSDF);		// IrisMask
	}
	else if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_HAIR)
	{
		Out = HAIR_BACKLIT(BSDF);			// BackLit
	}
	return Out;
}

float3 SubstrateGetBSDFEmissive(in FSubstrateBSDF BSDF)
{	
	return BSDF_GETEMISSIVE(BSDF);
}

float3 SubstrateGetWorldNormal(in FSubstratePixelHeader Header, in FSubstrateBSDF BSDF, in const FSubstrateAddressing Addressing)
{
	return SubstrateGetBSDFSharedBasis_DeferredShading(Header, BSDF, Addressing)[2];
}

float3 SubstrateGetWorldTangent(in FSubstratePixelHeader Header, in FSubstrateBSDF BSDF, in const FSubstrateAddressing Addressing)
{
	return SubstrateGetBSDFSharedBasis_DeferredShading(Header, BSDF, Addressing)[0];
}

float SubstrateGetAO(in FSubstratePixelHeader Header)
{
	return SubstrateUnpackIrradianceAndOcclusion(HEADER_GETIRRADIANCE_AO(Header.State)).MaterialAO;
}

FSubstrateIrradianceAndOcclusion SubstrateGetIrradianceAndAO(in FSubstratePixelHeader Header)
{
	return SubstrateUnpackIrradianceAndOcclusion(HEADER_GETIRRADIANCE_AO(Header.State));
}

uint SubstrateGetPackedIrradianceAndAO(in FSubstratePixelHeader Header)
{
	return HEADER_GETIRRADIANCE_AO(Header.State);
}

float SubstrateGetLegacyShadingModels(in FSubstrateBSDF BSDF)
{
	switch (BSDF_GETTYPE(BSDF))
	{
	case SUBSTRATE_BSDF_TYPE_SLAB:
	{
		// Based on features, we can only partially recovers the original shading models. Shading models like foliage, eye, clear coat are impossible/harder to infer.
		if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)	{ return SHADINGMODELID_SUBSURFACE_PROFILE; }
		else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_WRAP)			{ return SHADINGMODELID_SUBSURFACE; }
		else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_TWO_SIDED_WRAP)	{ return SHADINGMODELID_TWOSIDED_FOLIAGE; }
		else if (BSDF_GETHASFUZZ(BSDF))								{ return SHADINGMODELID_CLOTH; }
		else
		{ 
			if (BSDF_GETHASHAZINESS(BSDF))
			{
				FHaziness LocalHaziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
				if (LocalHaziness.bSimpleClearCoat)
				{
					return SHADINGMODELID_CLEAR_COAT;
				}
			}
			return SHADINGMODELID_DEFAULT_LIT;
		} // Also include SSS_TYPE_DIFFUSION and SSS_TYPE_SIMPLEVOLUME
	}
	case SUBSTRATE_BSDF_TYPE_HAIR: 								{ return SHADINGMODELID_HAIR; }
	case SUBSTRATE_BSDF_TYPE_EYE: 								{ return SHADINGMODELID_EYE; }
	case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:					{ return SHADINGMODELID_SINGLELAYERWATER; }
	case SUBSTRATE_BSDF_TYPE_UNLIT:								{ return SHADINGMODELID_UNLIT; } // This in fact does not show up because unlit does not write any information as it is not needed for any lighting passes down the line.
	}
	return 0.f;
}

///////////////////////////////////////////////////////////////////////////////
// BSDFs packing/unpacking
// 11G11B10F and 10F are unsigned float format

float3 sqrtFast3(float3 In) { return float3(sqrtFast(In.x), sqrtFast(In.y), sqrtFast(In.z)); }

half3 SubstrateLinearToSrgb(half3 In)
{
#if SUBSTRATE_SHADING_QUALITY >= 1
	return LinearToSrgb(In);
#else
	return sqrtFast3(In); // Simple gamma 2.0
#endif
}

half3 SubstrateSrgbToLinear(half3 In)
{
#if SUBSTRATE_SHADING_QUALITY >= 1
	return sRGBToLinear(In);
#else
	return In * In; // Simple gamma 2.0
#endif
}

uint PackColorLinearToGamma2(float3 In)
{
	return PackRGBA8(float4(SubstrateLinearToSrgb(saturate(In.rgb)), 0));
}

float3 UnpackColorGamma2ToLinear(uint In)
{
	float3 Out = UnpackRGBA8(In).xyz;
	return SubstrateSrgbToLinear(Out.rgb);
}

uint PackColorLinearToGamma2AlphaLinear(float4 In)
{
	return PackRGBA8(float4(SubstrateLinearToSrgb(saturate(In.rgb)), In.a));
}

float4 UnpackColorGamma2ToLinearAlphaLinear(uint In)
{
	float4 Out = UnpackRGBA8(In);
	return float4(SubstrateSrgbToLinear(Out.rgb), Out.a);
}

uint PackR6G7B6Gamma2(float3 rgb, float Dither)
{
	// We execute a dither that can go in positive/negative direction to avoid surfaces to looks overall brighter.
	const float CenteredDither = Dither - 0.5f;
	const float Dither8bits = CenteredDither * 1.0f / 255.0f;
	const float Dither7bits = CenteredDither * 1.0f / 127.0f;
	const float Dither6bits = CenteredDither * 1.0f / 63.0f;
	const float3 Dither676 = float3(Dither6bits, Dither7bits, Dither6bits);

	// Dither value to hide more aggressive bit reduction.
	const float3 dithered_rgb = DitherIfNonBlack(SubstrateLinearToSrgb(saturate(rgb)), Dither676);

	uint r = (uint(dithered_rgb.r * 63.0f ) <<  0);
	uint g = (uint(dithered_rgb.g * 127.0f) <<  6);
	uint b = (uint(dithered_rgb.b * 63.0f ) << 13);
	return r | g | b;
}

float3 UnpackR6G7B6Gamma2(uint rgb)
{
	float3 Out;
	Out.r = float((rgb >>  0) & 0x3F) * (1.0f / 63.0f);
	Out.g = float((rgb >>  6) & 0x7F) * (1.0f / 127.0f);
	Out.b = float((rgb >> 13) & 0x3F) * (1.0f / 63.0f);

	return SubstrateSrgbToLinear(Out);
}

uint PackR7G7B6Gamma2(float3 rgb, float Dither)
{
	// We execute a dither that can go in positive/negative direction to avoid surfaces to looks overall brighter.
	const float CenteredDither = Dither - 0.5f;
	const float Dither8bits = CenteredDither * 1.0f / 255.0f;
	const float Dither7bits = CenteredDither * 1.0f / 127.0f;
	const float Dither6bits = CenteredDither * 1.0f / 63.0f;
	const float3 Dither776 = float3(Dither7bits, Dither7bits, Dither6bits);

	// Dither value to hide more aggressive bit reduction.
	const float3 dithered_rgb = DitherIfNonBlack(SubstrateLinearToSrgb(saturate(rgb)), Dither776);

	uint r = (uint(dithered_rgb.r * 127.0f) << 0);
	uint g = (uint(dithered_rgb.g * 127.0f) << 7);
	uint b = (uint(dithered_rgb.b *  63.0f) << 14);
	return r | g | b;
}

float3 UnpackR7G7B6Gamma2(uint rgb)
{
	float3 Out;
	Out.r = float((rgb >> 0)  & 0x7F) * (1.0f / 127.0f);
	Out.g = float((rgb >> 7)  & 0x7F) * (1.0f / 127.0f);
	Out.b = float((rgb >> 14) & 0x3F) * (1.0f /  63.0f);

	return SubstrateSrgbToLinear(Out);
}

uint PackR6G6B5Gamma2(float3 rgb, float Dither)
{
	// We execute a dither that can go in positive/negative direction to avoid surfaces to looks overall brighter.
	const float CenteredDither = Dither - 0.5f;
	const float Dither6bits = CenteredDither * 1.0f / 63.0f;
	const float Dither5bits = CenteredDither * 1.0f / 31.0f;
	const float3 Dither665 = float3(Dither6bits, Dither6bits, Dither5bits);

	// Dither value to hide more aggressive bit reduction.
	const float3 dithered_rgb = DitherIfNonBlack(SubstrateLinearToSrgb(saturate(rgb)), Dither665);

	uint r = (uint(dithered_rgb.r * 63.0f) << 0);
	uint g = (uint(dithered_rgb.g * 63.0f) << 6);
	uint b = (uint(dithered_rgb.b * 31.0f) << 12);
	return r | g | b;
}

float3 UnpackR6G6B5Gamma2(uint rgb)
{
	float3 Out;
	Out.r = float((rgb >> 0)  & 0x3F) * (1.0f / 63.0f);
	Out.g = float((rgb >> 6)  & 0x3F) * (1.0f / 63.0f);
	Out.b = float((rgb >> 12) & 0x1F) * (1.0f / 31.0f);

	return SubstrateSrgbToLinear(Out);
}

uint PackR7G7B6Linear(float3 rgb, float Dither)
{
	// We execute a dither that can go in positive/negative direction to avoid surfaces to looks overall brighter.
	const float CenteredDither = Dither - 0.5f;
	const float Dither8bits = CenteredDither * 1.0f / 255.0f;
	const float Dither7bits = CenteredDither * 1.0f / 127.0f;
	const float Dither6bits = CenteredDither * 1.0f / 63.0f;
	const float3 Dither776 = float3(Dither7bits, Dither7bits, Dither6bits);

	// Dither value to hide more aggressive bit reduction.
	const float3 dithered_rgb = DitherIfNonBlack(saturate(rgb), Dither776);

	uint r = (uint(dithered_rgb.r * 127.0f) << 0);
	uint g = (uint(dithered_rgb.g * 127.0f) << 7);
	uint b = (uint(dithered_rgb.b *  63.0f) << 14);
	return r | g | b;
}

float3 UnpackR7G7B6Linear(uint rgb)
{
	float3 Out;
	Out.r = float((rgb >> 0)  & 0x7F) * (1.0f / 127.0f);
	Out.g = float((rgb >> 7)  & 0x7F) * (1.0f / 127.0f);
	Out.b = float((rgb >> 14) & 0x3F) * (1.0f /  63.0f);
	return Out;
}

uint PackR7(float Value, float Dither)
{
	return uint(Dither7bits(Value, Dither) * 127.0f);
}

float UnpackR7(uint Value)
{
	return float(Value & 0x7F) * (1.0f / 127.0f);
}

uint PackR6(float Value, float Dither)
{
	return uint(Dither6bits(Value, Dither) * 63.0f);
}

uint PackR5(float Value, float Dither)
{
	return uint(Dither5bits(Value, Dither) * 31.0f);
}

// Simple Octahedral from "A Survey of Efficient Representations for Independent Unit Vectors" http://jcgt.org/published/0003/02/01/
float SignNotZero(in float k)
{
	return k >= 0.0 ? 1.0 : -1.0;
}
float2 SignNotZero(in float2 v)
{
	return float2(SignNotZero(v.x), SignNotZero(v.y));
}

uint SubstratePackNormal(in float3 Normal)
{
	return PackFloat2ToUInt(UnitVectorToOctahedron(Normal));
}

float3 SubstrateUnpackNormal(uint PackedNormal)
{
	return OctahedronToUnitVector(UnpackFloat2FromUInt(PackedNormal));
}

uint SubstratePackNormal24(in float3 Normal)
{
	const float2 Result = UnitVectorToOctahedron(Normal);
	const uint2 PackedXY = uint2(clamp(Result * 2047.0f + 2048.0f, 0.0f, 4095.0f));
	return PackedXY.x | (PackedXY.y << 12);
}

float3 SubstrateUnpackNormal24(uint PackedNormal)
{
	const int2 XY12Bits = int2(0xFFF & PackedNormal, 0xFFF & (PackedNormal >> 12));
	const float2 xy = float2(XY12Bits - 2048) / 2047.0f;
	return OctahedronToUnitVector(xy);
}

uint SubstratePackNormal22(in float3 Normal)
{
	const float2 Result = UnitVectorToOctahedron(Normal);
	const uint2 PackedXY = uint2(clamp(Result * 1023.0f + 1024.0f, 0.0f, 2047.0f));
	return PackedXY.x | (PackedXY.y << 11);
}

float3 SubstrateUnpackNormal22(uint PackedNormal)
{
	const int2 XY11Bits = int2(0x7FF & PackedNormal, 0x7FF & (PackedNormal >> 11));
	const float2 xy = float2(XY11Bits - 1024) / 1023.0f;
	return OctahedronToUnitVector(xy);
}


// 11:11:10 encoding of normal + tangent angle
uint SubstratePackNormalAndTangent(in float3 Normal, in float3 Tangent)
{
	// Oct. encoding for the normal
	float2 OctF = UnitVectorToOctahedron(Normal);
	uint XBits = uint(round(OctF.x * 1023.0f + 1023.0f));
	uint YBits = uint(round(OctF.y * 1023.0f + 1023.0f));

	// Oct. encoding for the projected tangent, relative to a generated ortho frame based on decoded normal
	float3x3 Basis = GetTangentBasis(OctahedronToUnitVector(float2(XBits, YBits) / 1023.0f - 1.0f));
	float TX = dot(Basis[0], Tangent);
	float TY = dot(Basis[1], Tangent);
	float PseudoAngle = abs(TY) / (abs(TX) + abs(TY)); // Coarse ArcTan, returns value in [0,1]
	uint ABits = uint(round(PseudoAngle * 255.0f));
	// Merge all quantized quantities and sign bits of the tangent
	return XBits | (YBits << 11) | (ABits << 22) | ((asuint(TX) >> 1) & 0x40000000u) | (asuint(TY) & 0x80000000u);
}

void SubstrateUnpackNormalAndTangent(inout float3 Normal, inout float3 Tangent, in uint InPacked)
{
	uint XBits  = (InPacked      ) & 2047;
	uint YBits  = (InPacked >> 11) & 2047;
	uint ABits  = (InPacked >> 22) & 255;

	Normal = OctahedronToUnitVector(float2(XBits, YBits) / 1023.0f - 1.0f);
	float3x3 Basis = GetTangentBasis(Normal);
	float Y = float(ABits) / 255.0f;
	float2 P = float2(1.0f - Y, Y);
	// restore sign bits
	P.x = asfloat(asuint(P.x) | ((InPacked << 1) & 0x80000000u));
	P.y = asfloat(asuint(P.y) | ((InPacked     ) & 0x80000000u));
	P = normalize(P);
	Tangent = Basis[0] * P.x + Basis[1] * P.y;
}

void SubstrateUnpackColorGamma2ToLinearAlphaLinear(uint RawData, inout float3 RGB, inout float A)
{
	float4 Data = UnpackColorGamma2ToLinearAlphaLinear(RawData);
	RGB = Data.rgb;
	A = Data.a;
}

///////////////////////////////////////////////////////////////////////////////
// Substrate deferred base-pass output

#if SUBSTRATE_INLINE_SHADING && SUBSTRATE_CLAMPED_CLOSURE_COUNT > 0

// Update all Coverage/Transmittance/Weights for all BSDFs based on material topology
void FSubstratePixelHeader::SubstrateUpdateTree(
	float3 V,
	FSubstrateIntegrationSettings Settings)
{
	#if ALLOW_SUBSTRATE_TREE
	// bStaticBoolUseFullySimplifiedPath is meant to be staticalyl assigfned so that only one of the path below is ever compiled.
#if SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 1
	{
		//
		// Pre-Update the BSDF by traversing up the Substrate tree until the root node. This allows to modify BSDF's parameters based on operators (i.e., thin-film coating)
		//
		this.PreUpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified(V);

		//
		// Update the coverage/transmittance of each leaves (==BSDFs) of the Substrate tree.
		//
		this.UpdateAllBSDFsOperatorCoverageTransmittance_FullySimplified(Settings, V);

		//
		// Propagate up the coverage/transmittance of each node in the Substrate tree.
		//
		this.UpdateAllOperatorsCoverageTransmittance_FullySimplified();

		//
		// Update the luminance weight of each BSDF according to the operators it has to traverse up to the Substrate tree root node.
		//
		this.UpdateAllBSDFWithBottomUpOperatorVisit_FullySimplified();
	}
#else // Substrate_USE_FULLYSIMPLIFIED_MATERIAL
	{
		//
		// Pre-Update the BSDF by traversing up the Substrate tree until the root node. This allows to modify BSDF's parameters based on operators (i.e., thin-film coating)
		//
		this.PreUpdateAllBSDFWithBottomUpOperatorVisit(V);

		//
		// Update the coverage/transmittance of each leaves (==BSDFs) of the Substrate tree.
		//
		this.UpdateAllBSDFsOperatorCoverageTransmittance(Settings, V);

		//
		// Propagate up the coverage/transmittance of each node in the Substrate tree.
		//
		this.UpdateAllOperatorsCoverageTransmittance();

		//
		// Update the luminance weight of each BSDF according to the operators it has to traverse up to the Substrate tree root node.
		//
		this.UpdateAllBSDFWithBottomUpOperatorVisit();
	}
#endif // SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
	#endif // ALLOW_SUBSTRATE_TREE
}

void FSubstratePixelHeader::SubstrateUpdateTree(
	in FSubstrateData SubstrateData,		// This could be int RootOperatorIndex
	float3 V,
	FSubstrateIntegrationSettings Settings,
	inout float OutCoverage,
	inout float3 OutTransmittancePreCoverage)
{
	#if ALLOW_SUBSTRATE_TREE
	this.SubstrateUpdateTree(
		V,
		Settings);

	OutCoverage = saturate(SubstrateTree.Operators[SubstrateData.OperatorIndex].Coverage);
	OutTransmittancePreCoverage = saturate(SubstrateTree.Operators[SubstrateData.OperatorIndex].ThroughputAlongV);
	#endif
}

#endif // SUBSTRATE_INLINE_SHADING && SUBSTRATE_CLAMPED_CLOSURE_COUNT > 0

///////////////////////////////////////////////////////////////////////////////
// Header and BSDF packing

// Pack only Substrate header
uint PackSubstrateHeader(uint InBSDFCount, FSubstratePixelHeader InHeader)
{
	uint Out = 0;
	HEADER_SETBSDFCOUNT(Out, InBSDFCount);

#if SUBSTRATE_INLINE_SHADING
	HEADER_SETSHAREDLOCALBASESCOUNT(Out, InHeader.SharedLocalBases.Count);
	HEADER_SETSHAREDLOCALBASESTYPE(Out, InHeader.SharedLocalBases.Types);
#endif

#if SUBSTRATE_COMPLEXPATH
	Out |= InHeader.State & HEADER_COMPLEXPATHMODE_MASK;
#endif

	HEADER_SETCOMMONSTATES(Out, InHeader.State);

	return Out;
}

// Unpack only Substrate header
// If this is changed, please update the compiler side material size evaluation in SubstrateMaterial.cpp
FSubstratePixelHeader UnpackSubstrateHeaderIn(uint PackedHeader, inout FSubstrateAddressing SubstrateAddressing, FSubstrateTopLayerDataContainer InSubstrateTopLayerTexture)
{
	FSubstratePixelHeader Out = InitialiseSubstratePixelHeader();
	Out.State		= HEADER_GETCOMMONSTATES(PackedHeader);	// Unpack state and AO data at the same time 
	Out.ClosureCount= HEADER_GETBSDFCOUNT(PackedHeader);

	// Use simple or single layout for decoding data
	BRANCH
	if (Out.IsSimpleMaterial() || Out.IsSingleMaterial() || Out.IsEye() || Out.IsHair() || Out.IsSingleLayerWater())
	{
		// Within a tile classified as simple or single, some pixel might be invalid (e.g., sky pixels). 
		// This ensures that invalid pixels are not processed, and keep these tiles as single/simple instead of complex
		Out.ClosureCount = Out.IsSubstrateMaterial() ? 1 : 0; 

	  #if SUBSTRATE_DEFERRED_SHADING
		Out.PackedHeader = PackedHeader;
		Out.SharedLocalBasesIndexOffset = 0;	// Unused

		Out.PackedTopLayerData = SubstrateLoadTopLayerData(InSubstrateTopLayerTexture, SubstrateAddressing);
		SubstrateAddressing.ReadBytes += 4;
	  #endif
	}
	#if SUBSTRATE_COMPLEXPATH
	else
	{
		uint SharedLocalBasesCount = HEADER_GETSHAREDLOCALBASESCOUNT(PackedHeader);
		uint SharedLocalBasesTypes = HEADER_GETSHAREDLOCALBASESTYPE(PackedHeader);

		Out.SetComplexPathMode(HEADER_GETCOMPLEXPATHMODE(PackedHeader));

	#if SUBSTRATE_DEFERRED_SHADING
		Out.PackedHeader = PackedHeader;
		Out.PackedTopLayerData = 0;

		// Only keep the offset to the shared local bases memory. We are going to load them on demand.
		Out.SharedLocalBasesIndexOffset = SubstrateAddressing.CurrentIndex;

		// Now skip over the shared local basis memory to be able to load the BSDF content
		SubstrateAddressing.CurrentIndex += SharedLocalBasesCount;

		// And account for the space used by normal for the debug information
		SubstrateAddressing.ReadBytes += SharedLocalBasesCount * SUBSTRATE_PACKED_SHAREDLOCALBASIS_STRIDE_BYTES;
	#endif
	}
	#endif // SUBSTRATE_COMPLEXPATH

	return Out;
}

FSubstratePixelHeader UnpackSubstrateHeaderIn(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing, FSubstrateTopLayerDataContainer InSubstrateTopLayerTexture)
{
	uint PackedHeader = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
	FSubstratePixelHeader Out = UnpackSubstrateHeaderIn(PackedHeader, SubstrateAddressing, InSubstrateTopLayerTexture);

#if SUBSTRATE_DEFERRED_SHADING
	Out.SubstrateBuffer = SubstrateBuffer;
#endif

	return Out;
}

bool IsSubstrateSlabCompatible_SimplePath(in FSubstrateBSDF BSDF)
{
	//return (BSDF.State & SLAB_NOT_SINGLE_PATH_MASK) == 0;
	return (BSDF.State & (SLAB_SINGLE_PATH_MASK | SLAB_COMPLEX_PATH_MASK | SLAB_COMPLEX_SPECIAL_PATH_MASK)) == 0;
}

bool IsSubstrateSlabCompatible_SinglePath(in FSubstrateBSDF BSDF)
{
	return (BSDF.State & (SLAB_COMPLEX_PATH_MASK | SLAB_COMPLEX_SPECIAL_PATH_MASK)) == 0;
}

bool IsSubstrateSlabCompatible_ComplexPath(in FSubstrateBSDF BSDF)
{
	return (BSDF.State & (SLAB_COMPLEX_SPECIAL_PATH_MASK)) == 0;
}

bool IsSubstrateSlabCompatible_ComplexSpecialPath(in FSubstrateBSDF BSDF)
{
	return true;
}

void PackDiffuseAlbedoF0(float Dither, float3 DiffuseAlbedo, float3 F0, inout uint OutData0, inout uint OutData1)
{
	const uint PackedDiffuse20Bits	= PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
	const uint PackedF020Bits		= PackR7G7B6Gamma2(F0, Dither);

	OutData0 = BitFieldExtractU32(PackedF020Bits,  8,12) << 24;
	OutData1 =(BitFieldExtractU32(PackedF020Bits, 12, 0) << 20) | PackedDiffuse20Bits;

}

void PackDiffuseAlbedoF0Roughness(float Dither, float3 DiffuseAlbedo, float3 F0, float Roughness, inout uint OutData0, inout uint OutData1)
{
	PackDiffuseAlbedoF0(Dither, DiffuseAlbedo, F0, OutData0, OutData1);

	const uint PackedRoughness8Bits	= PackR8(Roughness);
	OutData0 = OutData0 | (PackedRoughness8Bits << 16);
}

void UnpackDiffuseAlbedoF0Roughness(uint InData0, uint InData1, inout float3 OutDiffuseAlbedo, inout float3 OutF0, inout float OutRoughness)
{
	const uint PackedRoughness8bits = BitFieldExtractU32(InData0, 8, HEADER_FASTENCODING_BIT_COUNT);
	const uint PackedDiffuse20Bits	= BitFieldExtractU32(InData1,20, 0);
	const uint PackedF020Bits		= BitFieldExtractU32(InData1,12, 20) | (BitFieldExtractU32(InData0, 8, HEADER_FASTENCODING_BIT_COUNT + 8)<<12);

	OutRoughness = UnpackR8(PackedRoughness8bits);
	OutDiffuseAlbedo = UnpackR7G7B6Gamma2(PackedDiffuse20Bits);
	OutF0 = UnpackR7G7B6Gamma2(PackedF020Bits);
}

void UnpackDiffuseAlbedoF0(uint InData0, uint InData1, inout float3 OutDiffuseAlbedo, inout float3 OutF0)
{
	float OutDummyRoughness;
	UnpackDiffuseAlbedoF0Roughness(InData0, InData1, OutDiffuseAlbedo, OutF0, OutDummyRoughness);
}

void FSubstrateBSDF::PackSingleOrComplexPathSlabBSDF(float Dither, inout uint OutData1, inout uint OutData2)
{
	const uint PackedDiffuse20Bits	= PackR7G7B6Gamma2(SLAB_DIFFUSEALBEDO(this), Dither);
	const uint PackedF020Bits		= PackR7G7B6Gamma2(SLAB_F0(this), Dither);

	OutData1 =(BitFieldExtractU32(PackedF020Bits, 12, 0) << 20) | PackedDiffuse20Bits;
	OutData2 = BitFieldExtractU32(PackedF020Bits, 8, 12) | PackRGBA8(float4(0.f, SLAB_ROUGHNESS(this), (SLAB_ANISOTROPY(this) + 1.f) * 0.5f, (SLAB_SSSPHASEANISOTROPY(this) + 1.f) * 0.5f));
}

void FSubstrateBSDF::UnpackSingleOrComplexPathSlabBSDF(uint Data1, uint Data2)
{
	const uint PackedDiffuse20Bits	= BitFieldExtractU32(Data1, 20, 0);
	const uint PackedF020Bits		= BitFieldExtractU32(Data1, 12,20) | (BitFieldExtractU32(Data2, 8, 0)<<12);

	SLAB_DIFFUSEALBEDO(this)	= UnpackR7G7B6Gamma2(PackedDiffuse20Bits);
	SLAB_F0(this)				= UnpackR7G7B6Gamma2(PackedF020Bits);

	float4 fData2 = UnpackRGBA8(Data2);
	SLAB_ROUGHNESS(this)			= fData2.y;
	SLAB_ANISOTROPY(this)			= fData2.z * 2.f - 1.f;
	SLAB_SSSPHASEANISOTROPY(this)	= fData2.w * 2.f - 1.f;
}

FSubstrateBSDF UnpackFastPathSubstrateBSDFIn(uint Data0, uint Data1)
{
	FSubstrateBSDF OutBSDF = (FSubstrateBSDF)0;

#if SUBSTRATE_DEFERRED_SHADING
	OutBSDF.State = 0;
	OutBSDF.LuminanceWeightV = 1.0f;
	OutBSDF.CoverageAboveAlongN = 0.0f;
	OutBSDF.TransmittanceAboveAlongN = 1.0f;

	float3 OutDiffuseAlbedo;
	float3 OutF0; 
	float OutRoughness;
	UnpackDiffuseAlbedoF0Roughness(Data0, Data1, OutDiffuseAlbedo, OutF0, OutRoughness);

	SLAB_DIFFUSEALBEDO(OutBSDF)	= OutDiffuseAlbedo;
	SLAB_F0(OutBSDF)			= OutF0;
	SLAB_ROUGHNESS(OutBSDF)		= OutRoughness;
	SLAB_ANISOTROPY(OutBSDF)	= 0.0f;
#endif

	// This is convenient to not have to test BSDF_GETHASF90 all over the place in the code using the BSDF for such a core value.
	SLAB_F90(OutBSDF) = 1.0f;

	// Force a BSDF state to a single slab with isotropic specular, no subsurface scattering or any other options.
	// This is to help the compiler understand disabled path.
	BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SLAB);
	BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
	BSDF_SETHASF90(OutBSDF, 0);
	BSDF_SETHASANISOTROPY(OutBSDF, 0);
	BSDF_SETISTOPLAYER(OutBSDF, 1);
	BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_NONE);
	BSDF_SETHASHAZINESS(OutBSDF, 0);
	BSDF_SETISTHIN(OutBSDF, 0);
	BSDF_SETHASFUZZ(OutBSDF, 0);
	BSDF_SETHASTRANSABOVE(OutBSDF, 0);
	BSDF_SETHASGLINT(OutBSDF, 0);

	return OutBSDF;
}

FSubstrateBSDF UnpackFastPathSubstrateBSDFIn(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing, FSubstratePixelHeader SubstratePixelHeader)
{
	#if SUBSTRATE_DEFERRED_SHADING
	uint Data0 = SubstratePixelHeader.PackedHeader;
	uint Data1 = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
	#else
	uint Data0 = 0;
	uint Data1 = 0;
	#endif

	return UnpackFastPathSubstrateBSDFIn(Data0, Data1);
}

FSubstrateBSDF UnpackFastWaterPathSubstrateBSDFIn(uint Data0, uint Data1)
{
	FSubstrateBSDF OutBSDF = (FSubstrateBSDF)0;

#if SUBSTRATE_DEFERRED_SHADING
	OutBSDF.State = 0;
	OutBSDF.LuminanceWeightV = 1.0f;
	OutBSDF.CoverageAboveAlongN = 0.0f;
	OutBSDF.TransmittanceAboveAlongN = 1.0f;

	const uint PackedBaseColor8Bits					= 0xFF   & (Data0 >> (HEADER_FASTENCODING_BIT_COUNT + 8));
	const uint RoughnessAndUseSeparateDirLight8Bits	= 0xFF   & (Data0 >> HEADER_SLWENCODING_BIT_COUNT);
	
	const uint PackedBaseColor12Bits				= 0xFFF  & (Data1 >> 20);
	const uint PackedOpacityMetalSpec20Bits			= 0xFFFFF & Data1;
	const uint PackedBaseColor20Bits				= PackedBaseColor12Bits | (PackedBaseColor8Bits << 12);

	const bool bStoreSeparatedMainDirLight			= RoughnessAndUseSeparateDirLight8Bits & 0x1;

	SLW_ROUGHNESS(OutBSDF)							= UnpackR7(RoughnessAndUseSeparateDirLight8Bits >> 1);
	SLW_BASECOLOR(OutBSDF)							= UnpackR7G7B6Gamma2(PackedBaseColor20Bits);

	const float3 OpacityMetalSpec					= UnpackR7G7B6Linear(PackedOpacityMetalSpec20Bits);
	SLW_TOPMATERIALOPACITY(OutBSDF)					= OpacityMetalSpec.x;
	SLW_METALLIC(OutBSDF)							= OpacityMetalSpec.y;
	SLW_SPECULAR(OutBSDF)							= OpacityMetalSpec.z;
#endif

	// Force a BSDF state to be a single layer on top.
	BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER);
	BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
	BSDF_SETHASF90(OutBSDF, 0);
	BSDF_SETHASANISOTROPY(OutBSDF, 0);
	BSDF_SETISTOPLAYER(OutBSDF, 1);
	BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_NONE);
	BSDF_SETHASHAZINESS(OutBSDF, 0);
	BSDF_SETISTHIN(OutBSDF, 0);
	BSDF_SETHASFUZZ(OutBSDF, 0);
	BSDF_SETHASTRANSABOVE(OutBSDF, 0);
	BSDF_SETHASGLINT(OutBSDF, 0);

	return OutBSDF;
}

FSubstrateBSDF UnpackFastWaterPathSubstrateBSDFIn(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing, FSubstratePixelHeader SubstratePixelHeader)
{
	#if SUBSTRATE_DEFERRED_SHADING
	uint Data0 = SubstratePixelHeader.PackedHeader;
	uint Data1 = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
	#else
	uint Data0 = 0;
	uint Data1 = 0;
	#endif

	return UnpackFastWaterPathSubstrateBSDFIn(Data0, Data1);
}
// Unpack a single BSDF
// Note: All BSDF lobes needs to be unpack in a sequential manner since each BSDF has a variable footprint.
// If this is changed, please update the compiler side material size evaluation in SubstrateMaterial.cpp
FSubstrateBSDF UnpackSubstrateBSDFIn(FSubstrateMaterialContainer SubstrateBuffer, inout FSubstrateAddressing SubstrateAddressing, in FSubstratePixelHeader SubstrateHeader, const bool bSkipSSSMaterialOverride = false)
{
	#if SUBSTRATE_DEFERRED_SHADING
	uint Data0 = SubstrateHeader.PackedHeader;
	uint Data1 = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
	#else
	uint Data0 = 0;
	uint Data1 = 0;
	#endif

	BRANCH
	if (SubstrateHeader.IsSimpleMaterial())
	{
		return UnpackFastPathSubstrateBSDFIn(Data0, Data1);
	}
	else if (SubstrateHeader.IsSingleLayerWater())
	{
		return UnpackFastWaterPathSubstrateBSDFIn(Data0, Data1);
	}

	FSubstrateBSDF OutBSDF = (FSubstrateBSDF)0;
	OutBSDF.CoverageAboveAlongN = 0.0f;
	OutBSDF.TransmittanceAboveAlongN = 1.0f;

	float DummyFloat = 0.0f;

	#if SUBSTRATE_DEFERRED_SHADING
	uint Data2 = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
	#else
	uint Data2 = 0;
	#endif

	BRANCH
	if (!SubstrateHeader.IsEye() && !SubstrateHeader.IsHair())
	{
	if (SubstrateHeader.IsSingleMaterial())
	{
		// Restore the BSDF common state
		OutBSDF.State = (Data0 >> (HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT)) & BSDF_SINGLEENCODING_MASK;

		// SUBSTRATE_TODO anisotropy need to be integrated in the optimised legacy single path.
		// For now, single encode doesn't support anisotropy, as the top layer data only contains the normal, not the full frame basis
		BSDF_SETHASANISOTROPY(OutBSDF, 0);

		uint OptimisedLegacyMode = (Data0 >> (HEADER_SINGLEENCODING_BIT_COUNT)) & HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK;
		if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_NONE)
		{
			// General single path
			OutBSDF.LuminanceWeightV = 1.0f;
			const uint BSDFType = SubstrateHeader.SubstrateGetBSDFType();
			BSDF_SETTYPE(OutBSDF, BSDFType);
		}
		else
		{
			OutBSDF.State = 0;
			OutBSDF.LuminanceWeightV = 1.0f;
			OutBSDF.CoverageAboveAlongN = 0.0f;
			OutBSDF.TransmittanceAboveAlongN = 1.0f;
			SLAB_F90(OutBSDF) = 1.0f;
	
			if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLEARCOAT)
			{
			#if SUBSTRATE_DEFERRED_SHADING
				float3 OutDiffuseAlbedo;
				float3 OutF0;
				UnpackDiffuseAlbedoF0(Data0, Data1, OutDiffuseAlbedo, OutF0);
				SLAB_DIFFUSEALBEDO(OutBSDF)	= OutDiffuseAlbedo;
				SLAB_F0(OutBSDF)			= OutF0;
				SLAB_ROUGHNESS(OutBSDF)		= SubstrateUnpackTopLayerData(SubstrateHeader.PackedTopLayerData).Roughness;

				const float ClearCoatRoughness = SLAB_ROUGHNESS(OutBSDF);
				const float BaseRoughness = GetHazinessRoughness(Data2);
				SLAB_HAZINESS(OutBSDF) = SetHazinessRoughness(Data2, ClearCoatRoughness);
				SLAB_ROUGHNESS(OutBSDF) = BaseRoughness;
			#endif

				// Force a BSDF state to be a single layer on top.
				BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SLAB);
				BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
				BSDF_SETHASF90(OutBSDF, 0);
				BSDF_SETHASANISOTROPY(OutBSDF, 0);
				BSDF_SETISTOPLAYER(OutBSDF, 1);
				BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_NONE);
				BSDF_SETHASHAZINESS(OutBSDF, 1);
				BSDF_SETISTHIN(OutBSDF, 0);
				BSDF_SETHASFUZZ(OutBSDF, 0);
				BSDF_SETHASTRANSABOVE(OutBSDF, 0);
				BSDF_SETHASGLINT(OutBSDF, 0);
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLOTH)
			{
			#if SUBSTRATE_DEFERRED_SHADING
				float3 OutDiffuseAlbedo;
				float3 OutF0;
				UnpackDiffuseAlbedoF0(Data0, Data1, OutDiffuseAlbedo, OutF0);
				SLAB_DIFFUSEALBEDO(OutBSDF)	= OutDiffuseAlbedo;
				SLAB_F0(OutBSDF)			= OutF0;
				SLAB_ROUGHNESS(OutBSDF)		= SubstrateUnpackTopLayerData(SubstrateHeader.PackedTopLayerData).Roughness;

				const float4 CustomData = UnpackRGBA8(Data2);
				SLAB_FUZZ_COLOR(OutBSDF)	= CustomData.xyz;
				SLAB_FUZZ_AMOUNT(OutBSDF)	= CustomData.w;
				SLAB_FUZZ_ROUGHNESS(OutBSDF)= SLAB_ROUGHNESS(OutBSDF);
			#endif

				// Force a BSDF state to be a single layer on top.
				BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SLAB);
				BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
				BSDF_SETHASF90(OutBSDF, 0);
				BSDF_SETHASANISOTROPY(OutBSDF, 0);
				BSDF_SETISTOPLAYER(OutBSDF, 1);
				BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_NONE);
				BSDF_SETHASHAZINESS(OutBSDF, 0);
				BSDF_SETISTHIN(OutBSDF, 0);
				BSDF_SETHASFUZZ(OutBSDF, 1);
				BSDF_SETHASTRANSABOVE(OutBSDF, 0);
				BSDF_SETHASGLINT(OutBSDF, 0);
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSWRAP || OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP)
			{
			#if SUBSTRATE_DEFERRED_SHADING
				float3 OutDiffuseAlbedo;
				float3 OutF0;
				UnpackDiffuseAlbedoF0(Data0, Data1, OutDiffuseAlbedo, OutF0);
				SLAB_DIFFUSEALBEDO(OutBSDF)	= OutDiffuseAlbedo;
				SLAB_F0(OutBSDF)			= OutF0;
				SLAB_ROUGHNESS(OutBSDF)		= SubstrateUnpackTopLayerData(SubstrateHeader.PackedTopLayerData).Roughness;

				const uint PackedOpacity7bit = BitFieldExtractU32(Data0, 5, HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT) | (BitFieldExtractU32(Data2, 2, 30) << 5);
				const float Opacity = UnpackR7(PackedOpacity7bit);

				SLAB_SSSMFP(OutBSDF)		= UnpackR10G10B10F(BitFieldExtractU32(Data2, 30, 0));
				SLAB_SSSPHASEANISOTROPY(OutBSDF)= saturate(1.f - Opacity);
			#endif

				const bool bTwoSidedWrap = OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP;

				// Force a BSDF state to be a single layer on top.
				BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SLAB);
				BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
				BSDF_SETHASF90(OutBSDF, 0);
				BSDF_SETHASANISOTROPY(OutBSDF, 0);
				BSDF_SETISTOPLAYER(OutBSDF, 1);
				BSDF_SETSSSTYPE(OutBSDF, bTwoSidedWrap ? SSS_TYPE_TWO_SIDED_WRAP : SSS_TYPE_WRAP);
				BSDF_SETHASMFP(OutBSDF, 1);
				BSDF_SETHASHAZINESS(OutBSDF, 0);
				BSDF_SETISTHIN(OutBSDF, 0);
				BSDF_SETHASFUZZ(OutBSDF, 0);
				BSDF_SETHASTRANSABOVE(OutBSDF, 0);
				BSDF_SETHASGLINT(OutBSDF, 0);
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSPROFILE)
			{
			#if SUBSTRATE_DEFERRED_SHADING
				float3 OutDiffuseAlbedo;
				float3 OutF0;
				UnpackDiffuseAlbedoF0(Data0, Data1, OutDiffuseAlbedo, OutF0);
				SLAB_DIFFUSEALBEDO(OutBSDF)	= OutDiffuseAlbedo;
				SLAB_F0(OutBSDF)			= OutF0;
				SLAB_ROUGHNESS(OutBSDF)		= SubstrateUnpackTopLayerData(SubstrateHeader.PackedTopLayerData).Roughness;

				SLAB_SSSPROFILEID(OutBSDF)			= float(BitFieldExtractU32(Data2, 8, 0));
				SLAB_SSSPROFILERADIUSSCALE(OutBSDF)	= UnpackG8(Data2);
				SLAB_HAZINESS(OutBSDF)				= BitFieldExtractU32(Data2, 16, 16);

				// When loading Substrate BSDF which have SSS enable, BaseColor and (optionally) Specular value can be overriden 
				// based on the SSS method used (screen space SSS / checherboarded screen space SSS / ...)
				#if SUBSTRATE_SSS_MATERIAL_OVERRIDE
				if(!bSkipSSSMaterialOverride)
				{
					const bool bChecker = CheckerFromPixelPos(SubstrateAddressing.PixelCoords);
					float3 DiffuseAlbedo = SLAB_DIFFUSEALBEDO(OutBSDF);
					float SpecularFactor = 1.0f;
					AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(DiffuseAlbedo, SpecularFactor, bChecker);
					SLAB_DIFFUSEALBEDO(OutBSDF) = DiffuseAlbedo;
					SLAB_F0(OutBSDF)  *= SpecularFactor;
					SLAB_F90(OutBSDF) *= SpecularFactor;
				}
				#endif
			#endif

				// Force a BSDF state to be a single layer on top.
				BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_SLAB);
				BSDF_SETHASGREYWEIGHT_V(OutBSDF, 1);
				BSDF_SETHASF90(OutBSDF, 0);
				BSDF_SETHASANISOTROPY(OutBSDF, 0);
				BSDF_SETISTOPLAYER(OutBSDF, 1);
				BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_DIFFUSION_PROFILE);
				BSDF_SETHASMFP(OutBSDF, 1);
				BSDF_SETHASHAZINESS(OutBSDF, 1);
				BSDF_SETISTHIN(OutBSDF, 0);
				BSDF_SETHASFUZZ(OutBSDF, 0);
				BSDF_SETHASTRANSABOVE(OutBSDF, 0);
				BSDF_SETHASGLINT(OutBSDF, 0);
			}
			return OutBSDF;
		}
	}


	// SingleNonOpt / Complex / ComplexSpecial
	{
		if (SubstrateHeader.IsSingleMaterial())
		{
			//OutBSDF.State = ... already done above
			#if SUBSTRATE_DEFERRED_SHADING
			OutBSDF.UnpackSingleOrComplexPathSlabBSDF(Data1, Data2);
			#endif
		}
		else
		{
			OutBSDF.State = Data1;  //SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			
			uint DataX, DataY;
			if (BSDF_GETHASGREYWEIGHT_V(OutBSDF))
			{
				OutBSDF.LuminanceWeightV = Unpack10F(BSDF_GETWEIGHT10F(OutBSDF));
				DataX = Data2;
				DataY = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			}
			else
			{
				// TODO this does not work with fixed layou loading
				//SubstrateLoad_R11G11B10F(SubstrateBuffer, SubstrateAddressing, OutBSDF.LuminanceWeightV);
				const uint RawData = Data2; //SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
				OutBSDF.LuminanceWeightV = UnpackR11G11B10F(RawData);

				DataX = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
				DataY = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			}

			#if SUBSTRATE_DEFERRED_SHADING
			if (SubstrateHeader.GetComplexPathMode() == COMPLEXPATH_MODE_TOON)
			{
				TOON_BASECOLOR(OutBSDF)	= UnpackR6G6B5Gamma2(DataX);
				TOON_SPECULAR(OutBSDF)	= UnpackR5(DataX >> 17);
				TOON_METALLIC(OutBSDF)	= UnpackR5(DataX >> 22);
				TOON_ROUGHNESS(OutBSDF)	= UnpackR5(DataX >> 27);
		
				UnpackToonCustomData(DataY, TOON_PATTERNUVS(OutBSDF), TOON_PROFILEID(OutBSDF));

				if (BSDF_GETHASANISOTROPY(OutBSDF))
				{
					TOON_ANISOTROPY(OutBSDF) = asfloat(SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing));
				}

				BSDF_SETSSSTYPE(OutBSDF, SSS_TYPE_TRANSMITTANCE_SHADOW);

				return OutBSDF;
			}
			else
			{
				OutBSDF.UnpackSingleOrComplexPathSlabBSDF(DataX, DataY);
			}
			#endif
		}

		SLAB_F90(OutBSDF) = 1.0f;
		const bool bHasF90 = BSDF_GETHASF90(OutBSDF);
		if (bHasF90 || BSDF_GETHASHAZINESS(OutBSDF))
		{
			const uint RawData = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);

			BRANCH
			if (bHasF90)
			{
				const float2 DataRGBA8 = UnpackRGBA8(RawData).xy;

				const float F90Co = DataRGBA8.x;
				const float F90Cg = DataRGBA8.y;
				// Setting Y=1 when converting back from YCoCg can lead to components being greater than 1. We fix this using two steps:
				// 1- We apply a simple scale determined manually on Y to keep Hue/Saturation expressivity large and similare to the input color from the base pass,
				//    even though the mapping from saturate to white is not linear.
				const float F90Y = 1.0f / 3.0f;
				float3 F90 = bHasF90 ? saturate(NormalisedYCoCg_2_LinearRGB(float3(F90Y, F90Co, F90Cg))) : 1.0f;
				// 2- We re-scale again the final recovered color according to its max component to maximise the brightness/hue similarity with the input color from the base pass.
				const float Divisor = max(F90.r, max(F90.g, F90.b));
				F90 = Divisor > 0.0f ? F90 / Divisor : F90;
				SLAB_F90(OutBSDF) = F90;

				SLAB_HAZINESS(OutBSDF) = RawData >> 16; // Keeping the top 16 bits only,
			}
			else
			{
				// See PackSubstrateOut for explanation of the special Haziness data write.
				SLAB_HAZINESS(OutBSDF) = RawData;
			}
		}

		const bool bHasSSS = BSDF_GETSSSTYPE(OutBSDF) != SSS_TYPE_NONE;
		if (bHasSSS)
		{
			const uint RawData = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			if (BSDF_GETSSSTYPE(OutBSDF) == SSS_TYPE_DIFFUSION_PROFILE)
			{
				UnpackSSSProfile(RawData, SLAB_SSSPROFILEID(OutBSDF), SLAB_SSSPROFILERADIUSSCALE(OutBSDF), SLAB_SSSPROFILETHICKNESSCM(OutBSDF));
			}
			else
			{
				SLAB_SSSMFP(OutBSDF) = UnpackR11G11B10F(RawData);
			}

			// When loading Substrate BSDF which have SSS enable, BaseColor and (optionally) Specular value can be overriden 
			// based on the SSS method used (screen space SSS / checherboarded screen space SSS / ...)
			#if SUBSTRATE_SSS_MATERIAL_OVERRIDE
			if (!bSkipSSSMaterialOverride && (BSDF_GETSSSTYPE(OutBSDF) == SSS_TYPE_DIFFUSION || BSDF_GETSSSTYPE(OutBSDF) == SSS_TYPE_DIFFUSION_PROFILE))
			{
				const bool bChecker = CheckerFromPixelPos(SubstrateAddressing.PixelCoords);
				float3 DiffuseAlbedo = SLAB_DIFFUSEALBEDO(OutBSDF);
				float SpecularFactor = 1.0f;
				AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(DiffuseAlbedo, SpecularFactor, bChecker);
				SLAB_DIFFUSEALBEDO(OutBSDF) = DiffuseAlbedo;
				SLAB_F0(OutBSDF)  *= SpecularFactor;
				SLAB_F90(OutBSDF) *= SpecularFactor;
			}
			#endif
		}

		if (BSDF_GETHASFUZZ(OutBSDF))
		{
			UnpackFuzz((SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing)), SLAB_FUZZ_COLOR(OutBSDF), SLAB_FUZZ_AMOUNT(OutBSDF), SLAB_FUZZ_ROUGHNESS(OutBSDF));
		}

		if (BSDF_GETHASTRANSABOVE(OutBSDF))
		{
			SubstrateUnpackColorGamma2ToLinearAlphaLinear(SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing), OutBSDF.TransmittanceAboveAlongN, OutBSDF.CoverageAboveAlongN);
		}

		#if SUBSTRATE_GLINTS_ENABLED
		if (BSDF_GETHASGLINT(OutBSDF))
		{
			uint2 GlintPackedData;
			GlintPackedData.x = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			GlintPackedData.y = SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing);
			UnpackGlints(GlintPackedData, SLAB_GLINT_VALUE(OutBSDF), SLAB_GLINT_UV(OutBSDF));

			SLAB_GLINT_UVDDX(OutBSDF)	= UnpackFloat2FromUInt(SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing));
			SLAB_GLINT_UVDDY(OutBSDF)	= UnpackFloat2FromUInt(SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing));
		}
		#endif // SUBSTRATE_GLINTS_ENABLED
		#if SUBSTRATE_SPECPROFILE_ENABLED
		if (BSDF_GETHASSPECPROFILE(OutBSDF))
		{
			UnpackSpecularProfile(SubstrateLoadUint1(SubstrateBuffer, SubstrateAddressing), SLAB_SPECPROFILEID(OutBSDF));
		}
		#endif // SUBSTRATE_SPECPROFILE_ENABLED
	}
	}
	else if (SubstrateHeader.IsEye())
	{
		OutBSDF.State = Data0 & HEADER_EYEENCODING_MASK;
		OutBSDF.LuminanceWeightV = 1.0f;
		BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_EYE);
		BSDF_SETSHAREDLOCALBASISID(OutBSDF, 0);
		BSDF_SETHASANISOTROPY(OutBSDF, 0);
		BSDF_SETISTOPLAYER(OutBSDF, 1);
		BSDF_SETSSSTYPE(OutBSDF, SubstrateHeader.HasSubsurface() ? SSS_TYPE_DIFFUSION_PROFILE : SSS_TYPE_NONE);

		float4 EyeData0 = 0;
		#if SUBSTRATE_DEFERRED_SHADING
		EyeData0 = UnpackRGBA8(Data0);
		#endif
		EYE_IRISMASK(OutBSDF) = EyeData0.z;
		EYE_IRISDISTANCE(OutBSDF) = EyeData0.w;

		SubstrateUnpackColorGamma2ToLinearAlphaLinear(Data1, EYE_DIFFUSEALBEDO(OutBSDF), EYE_ROUGHNESS(OutBSDF)); // REMOVE READ, use Data2
		const float4 EyeData2 = UnpackRGBA8(Data2);
		EYE_IRISNORMAL(OutBSDF) = OctahedronToUnitVector(EyeData2.xy * 2.0f - 1.f);
		EYE_IRISPLANENORMAL(OutBSDF) = OctahedronToUnitVector(EyeData2.zw * 2.0f - 1.f);
		EYE_F0(OutBSDF)  = SUBSTRATE_EYE_DEFAULT_F0;
		EYE_F90(OutBSDF) = 1.0f;

		// When loading Substrate BSDF which have SSS enable, BaseColor and (optionally) Specular value can be overriden 
		// based on the SSS method used (screen space SSS / checherboarded screen space SSS / ...)
		#if SUBSTRATE_SSS_MATERIAL_OVERRIDE
		if (!bSkipSSSMaterialOverride && BSDF_GETSSSTYPE(OutBSDF) == SSS_TYPE_DIFFUSION_PROFILE)
		{
			const bool bChecker = CheckerFromPixelPos(SubstrateAddressing.PixelCoords);
			float3 DiffuseAlbedo = EYE_DIFFUSEALBEDO(OutBSDF);
			float SpecularFactor = 1.0f;
			AdjustBaseColorAndSpecularColorForSubsurfaceProfileLighting(DiffuseAlbedo, SpecularFactor, bChecker);
			EYE_DIFFUSEALBEDO(OutBSDF) = DiffuseAlbedo;
			EYE_F0(OutBSDF)  *= SpecularFactor;
			EYE_F90(OutBSDF) *= SpecularFactor;
		}
		#endif
		// 8 bytes
	}
	else if (SubstrateHeader.IsHair())
	{
		OutBSDF.State = Data0 & HEADER_HAIRENCODING_MASK;
		OutBSDF.LuminanceWeightV = 1.0f;
		BSDF_SETTYPE(OutBSDF, SUBSTRATE_BSDF_TYPE_HAIR);
		BSDF_SETSHAREDLOCALBASISID(OutBSDF, 0);
		BSDF_SETHASANISOTROPY(OutBSDF, 0);
		BSDF_SETISTOPLAYER(OutBSDF, 1);

		SubstrateUnpackColorGamma2ToLinearAlphaLinear(Data1, HAIR_BASECOLOR(OutBSDF), HAIR_ROUGHNESS(OutBSDF)); // REMOVE READ, use Data2
		float4 HairData2 = UnpackRGBA8(Data2);
		HAIR_SCATTER(OutBSDF) = HairData2.x;
		HAIR_SPECULAR(OutBSDF) = HairData2.y;
		HAIR_BACKLIT(OutBSDF) = HairData2.z;
		HAIR_COMPLEXTRANSMITTANCE(OutBSDF) = HairData2.w;
		// 8 bytes
	}

	return OutBSDF;
}

#endif // SUBSTRATE_ENABLED
