// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/SubstrateDefinitions.h"   

#if PERMUTATION_TILETYPE == 0
#define SUBSTRATE_FASTPATH 1
#elif PERMUTATION_TILETYPE == 1
#define SUBSTRATE_SINGLEPATH 1
#elif PERMUTATION_TILETYPE == 2
  // COMPLEX PATH
#else
#error Substrate tile type non-implemented 
#endif

#define OpaqueBasePass

#define SUBSTRATE_MATERIALCONTAINER_IS_WRITABLE 1
#define SUBSTRATE_INLINE_SHADING 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
#include "/Engine/Private/Substrate/Substrate.ush"
#include "SubstrateTile.ush"


////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Post Dbuffer application
#if defined(DBufferPassCS) || defined(DBufferPassPS)
Texture2D<float4> DBufferATexture;
Texture2D<float4> DBufferBTexture;
Texture2D<float4> DBufferCTexture;
Texture2D<uint> DBufferRenderMask;

SamplerState DBufferATextureSampler;
SamplerState DBufferBTextureSampler;
SamplerState DBufferCTextureSampler;

#ifndef SUBSTRATE_STENCIL_DBUFFER_MASK
#error SUBSTRATE_STENCIL_DBUFFER_MASK needs to be defined
#endif
#ifndef STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID
#error STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID needs to be defined
#endif
#ifndef STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID
#error STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID needs to be defined
#endif
#ifndef STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID
#error STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID needs to be defined
#endif

// Sanity check
#if !defined(USE_DBUFFER) || USE_DBUFFER != 1
#error This shader should only be compiled when USE_DBUFFER == 1
#endif

#define MATERIALDECALRESPONSEMASK 0x7
#define ENABLE_DBUFFER_TEXTURES 1
#define DBUFFER_BASEPASS_RESOURCE 0

#define OpaqueBasePassDBufferATexture DBufferATexture
#define OpaqueBasePassDBufferBTexture DBufferBTexture
#define OpaqueBasePassDBufferCTexture DBufferCTexture
#define OpaqueBasePassDBufferRenderMask DBufferRenderMask

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define OpaqueBasePassDBufferATextureSampler DBufferATextureSampler
#define OpaqueBasePassDBufferBTextureSampler DBufferATextureSampler
#define OpaqueBasePassDBufferCTextureSampler DBufferATextureSampler
#else
#define OpaqueBasePassDBufferATextureSampler DBufferATextureSampler
#define OpaqueBasePassDBufferBTextureSampler DBufferBTextureSampler
#define OpaqueBasePassDBufferCTextureSampler DBufferCTextureSampler
#endif

#include "../DBufferDecalShared.ush"

uint GetDBufferResponseMask(uint2 PixelCoord)
{
	// Early out of no decal has been written or if the pixel has no 'decal responseness'
	const uint ResponseMask = SceneStencilTexture.Load(uint3(PixelCoord, 0)) STENCIL_COMPONENT_SWIZZLE;
	uint ValidDBufferTargetMask = GetDBufferTargetMask(PixelCoord);
	if (!ValidDBufferTargetMask || (ResponseMask & SUBSTRATE_STENCIL_DBUFFER_MASK) == 0)
	{
		return 0;
	}

	// Load Dbuffer data only attributes for which the material has some 'responseness'
	ValidDBufferTargetMask = ValidDBufferTargetMask & 
	(
		(((ResponseMask >> STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID)   & 0x1) << SUBSTRATE_DBUFFER_RESPONSE_BASECOLOR_BIT_OFFSET) |
		(((ResponseMask >> STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID)    & 0x1) << SUBSTRATE_DBUFFER_RESPONSE_NORMAL_BIT_OFFSET)    |
		(((ResponseMask >> STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID) & 0x1) << SUBSTRATE_DBUFFER_RESPONSE_ROUGHNESS_BIT_OFFSET)
	);

	return ValidDBufferTargetMask;
}

#endif // defined(DBufferPassCS) || defined(DBufferPassPS)

#ifdef DBufferPassCS

int2 ViewResolution;
uint MaxBytesPerPixel;
uint FirstSliceStoringSubstrateSSSData;
RWTexture2D<SUBSTRATE_TOP_LAYER_TYPE> TopLayerTexture;
RWTexture2DArray<uint> MaterialTextureArrayUAV;

uint TileEncoding;
uint TileListBufferOffset;
Buffer<uint> TileListBuffer;
Buffer<uint> TileIndirectBuffer;

// TODO
// * Add support Eye, Hair, Water, ... ?
// * Refactor encoding to share code between Substrate export

[numthreads(SUBSTRATE_TILE_SIZE, SUBSTRATE_TILE_SIZE, 1)]
void DBufferPassCS(
	uint2 DispatchThreadId : SV_DispatchThreadID, 
	uint LinearIndex : SV_GroupIndex, 
	uint3 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID)
{
	const uint2 TileCoord = SubstrateUnpackTile(TileListBuffer[TileListBufferOffset + GroupId.x], TileEncoding);
	const uint2 PixelCoord = TileCoord * SUBSTRATE_TILE_SIZE + GroupThreadId;
	const bool bIsValid = all(PixelCoord < uint2(View.ViewSizeAndInvSize.xy));

	if (!bIsValid)
	{
		return;
	}

	// Early out of no decal has been written or if the pixel has no 'decal responseness'
	const uint ValidDBufferTargetMask = GetDBufferResponseMask(PixelCoord);
	BRANCH
	if (ValidDBufferTargetMask == 0)
	{
		return;
	}

	const float2 BufferUV = SvPositionToBufferUV(float4(PixelCoord + 0.5f, 0, 0));
	const FSubstrateDBuffer DBufferData = SubstrateGetDBufferData(BufferUV, ValidDBufferTargetMask);

	// Early out if DBuffer has no coverage
	BRANCH
	if (DBufferData.Coverage <= 0)
	{
		return;
	}

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(MaterialTextureArrayUAV, SubstrateAddressing, TopLayerTexture);

	BRANCH
	if (SubstratePixelHeader.ClosureCount <= 0)
	{
		return;
	}

	BRANCH
	if (SubstratePixelHeader.IsSimpleMaterial() || SubstratePixelHeader.IsSingleMaterial())
	{
		// 1. Load BSDF data
		const float Dither = InterleavedGradientNoise(PixelCoord + 0.5f, View.StateFrameIndexMod8);
		FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(MaterialTextureArrayUAV, SubstrateAddressing, SubstratePixelHeader);

		// 2. Load DBuffer data and apply them
		float3 F0 = 0;
		float3 DiffuseAlbedo = 0;
		float Roughness = 0;
		{
			Roughness = SubstrateGetBSDFRoughness(BSDF);

			float3 BaseColor = 0;
			float Metallic = 0;
			float Specular = 0;
			ConvertToMetalness(SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), BaseColor, Specular, Metallic);

			Metallic  = Metallic  * DBufferData.OneMinusCoverage_Roughness + DBufferData.Metallic;
			Specular  = Specular  * DBufferData.OneMinusCoverage_Roughness + DBufferData.Specular;
			Roughness = Roughness * DBufferData.OneMinusCoverage_Roughness + DBufferData.Roughness;
			BaseColor = BaseColor * DBufferData.OneMinusCoverage_BaseColor + DBufferData.BaseColor;

			ConvertFromMetalness(BaseColor,	Specular, Metallic, DiffuseAlbedo, F0);
		}

		// 3. Top layer data
		BRANCH
		if (DBufferData.OneMinusCoverage_WorldNormal > 0 || DBufferData.OneMinusCoverage_Roughness > 0)
		{
			FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(SubstratePixelHeader.PackedTopLayerData);
			TopLayerData.WorldNormal = TopLayerData.WorldNormal * DBufferData.OneMinusCoverage_WorldNormal + DBufferData.WorldNormal;
			TopLayerData.Roughness = Roughness;
			TopLayerTexture[PixelCoord] = SubstratePackTopLayerData(TopLayerData);
		}

		// 4. Rencode data
		// SUBSTRATE_TODO: For now, doing manual reencoding by duplicating logic from Substrate export. Need to refactor Substrate export to have small reusable packing functions
		
		// Roughness / DiffuseColor / F0
		if (SubstratePixelHeader.IsSimpleMaterial())
		{
			const uint PackedDiffuse20Bits = PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
			const uint PackedDiffuse12Bits = PackedDiffuse20Bits & 0xFFF;
			const uint PackedDiffuse8Bits  = (PackedDiffuse20Bits >> 12) & 0xFF;
			const uint PackedF020Bits      = PackR7G7B6Gamma2(F0, Dither);
			const uint PackedRoughness8bits= PackR8(Roughness);

			// Data0 (Header_State|Header_AO|Roughness|Diffuse8bits)
			MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = (SubstratePixelHeader.PackedHeader & 0x0000FFFF) | (PackedRoughness8bits << 16) | (PackedDiffuse8Bits << 24);
			MaterialTextureArrayUAV[uint3(PixelCoord, 1)] = PackedF020Bits | (PackedDiffuse12Bits << 20);
		}
		#if !SUBSTRATE_FASTPATH
		else if (SubstratePixelHeader.IsSingleMaterial())
		{
			const uint OptimisedLegacyMode = (SubstratePixelHeader.PackedHeader >> (HEADER_SINGLEENCODING_BIT_COUNT)) & HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK;
			if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_NONE)
			{
				MaterialTextureArrayUAV[uint3(PixelCoord, 2)] = (MaterialTextureArrayUAV[uint3(PixelCoord, 2)] & 0xFFFF00FF) | (PackR8(Roughness) << 8);

				const uint PackedDiffuse20Bits	= PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
				const uint PackedF020Bits		= PackR7G7B6Gamma2(F0, Dither);
				const uint PackedData32Bits		= (PackedDiffuse20Bits << 12) | (PackedF020Bits & 0xFFF);
				const uint PackedData8Bits		= (PackedF020Bits >> 12) & 0xFF;

				uint RWOffset = 1;
				MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset++)] = PackedData32Bits; 
				MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset++)] = PackedData8Bits | PackRGBA8(float4(0.0f, Roughness, (SLAB_ANISOTROPY(BSDF) + 1.f) * 0.5f, (SLAB_SSSPHASEANISOTROPY(BSDF) + 1.f) * 0.5f));

				if (BSDF_GETHASF90(BSDF) || BSDF_GETHASHAZINESS(BSDF))
				{
					if (BSDF_GETHASHAZINESS(BSDF))
					{
						FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
						Haziness.Weight *= DBufferData.OneMinusCoverage_Roughness;
						const uint PackedHaziness = PackHaziness(Haziness);
						MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset)] = (MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset)] & 0x0000FFFF) | (PackedHaziness << 16);
					}

					++RWOffset;
				}

				const bool bHasSSS = BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE;
				if (bHasSSS)
				{
					// Disable SSS when the coverage reach one to preserve the decal color
					SLAB_SSSMFP(BSDF) *= DBufferData.OneMinusCoverage_BaseColor;
					if (DBufferData.OneMinusCoverage_BaseColor == 0.0f)
					{
						BSDF_SETSSSTYPE(BSDF, SSS_TYPE_NONE);
					}

					if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
					{
						// Nothing to do
					}
					else
					{
						// Path used for bottom most layer SSS, simple volume and two sided lighting.
						MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset)] = PackR11G11B10F(SLAB_SSSMFP(BSDF));
					}

					// Only write SSS data if needed
					if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION)
					{
						FSubstrateSubsurfaceExtras SSSDataExtras;
						SubstrateSubsurfaceExtrasSetBaseColor(SSSDataExtras, DiffuseAlbedo);
						SubstrateStoreSubsurfaceExtras(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSDataExtras.Bytes);
					}

					++RWOffset;
				}

				if (BSDF_GETHASFUZZ(BSDF))
				{
					MaterialTextureArrayUAV[uint3(PixelCoord, RWOffset)] = PackFuzz(
						SLAB_FUZZ_COLOR(BSDF), 
						SLAB_FUZZ_AMOUNT(BSDF) * DBufferData.OneMinusCoverage_BaseColor, 
						SLAB_FUZZ_ROUGHNESS(BSDF), Dither);
					RWOffset++;
				}
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLEARCOAT)
			{
				const uint InData2 = MaterialTextureArrayUAV[uint3(PixelCoord, 2)];

				// Clear coat disappear with coverage
				FHaziness Haziness = UnpackHaziness(InData2);
				Haziness.Weight *= DBufferData.OneMinusCoverage_BaseColor;
				Haziness.BottomNormalOct = lerp(DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT, Haziness.BottomNormalOct, DBufferData.OneMinusCoverage_BaseColor);
				const uint PackedHaziness = PackHaziness(Haziness);

				MaterialTextureArrayUAV[uint3(PixelCoord, 2)] = PackedHaziness;
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLOTH)
			{
				// Roughness is only stored in the TopLayer texture.

				// Diffuse Albedo & F0
				const uint PackedDiffuse20Bits   = PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
				const uint PackedF020Bits        = PackR7G7B6Gamma2(F0, Dither);
				const uint PackedFuzzColor20bits = PackR7G7B6Gamma2(SLAB_FUZZ_COLOR(BSDF), Dither);
				const uint PackedFuzzAmount8bits = PackR8(SLAB_FUZZ_AMOUNT(BSDF) * DBufferData.OneMinusCoverage_BaseColor);

				const uint FuzzAmountOffset = HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT;
				const uint FuzzAmountMask = (1u << FuzzAmountOffset) - 1u;
				MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = (MaterialTextureArrayUAV[uint3(PixelCoord, 0)] & FuzzAmountMask) | (PackedFuzzAmount8bits << FuzzAmountOffset);
				MaterialTextureArrayUAV[uint3(PixelCoord, 1)] = (PackedDiffuse20Bits | ((PackedFuzzColor20bits & 0xFFC00) << 10));
				MaterialTextureArrayUAV[uint3(PixelCoord, 2)] = (PackedF020Bits | ((PackedFuzzColor20bits & 0x3FF) << 20));
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSWRAP || OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP)
			{
				// Roughness is only stored in the TopLayer texture.
				const float3 MFP = SLAB_SSSMFP(BSDF) * DBufferData.OneMinusCoverage_BaseColor;

				// DiffuseAlbedo & F0
				const uint PackedDiffuse20Bits = PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
				const uint PackedF020Bits      = PackR7G7B6Gamma2(F0, Dither);
				const uint PackedSSSMFP30bits  = PackR10G10B10F(MFP);

				const uint PackedSSSMFP12BitsA = (PackedSSSMFP30bits) & 0xFFF; // 12 lower bits
				const uint PackedSSSMFP12BitsB = (PackedSSSMFP30bits >> 12) & 0xFFF; // next 12 lower bits
				const uint PackedSSSMFP6BitsC  = (PackedSSSMFP30bits >> 24) & 0x03F; // 6 higher bits

				const uint BitShift = (7 + HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT);

				MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = (MaterialTextureArrayUAV[uint3(PixelCoord, 0)] & 0x03FFFFFF) | (PackedSSSMFP6BitsC<< BitShift);
				MaterialTextureArrayUAV[uint3(PixelCoord, 1)] = PackedDiffuse20Bits | (PackedSSSMFP12BitsA << 20);
				MaterialTextureArrayUAV[uint3(PixelCoord, 2)] = PackedF020Bits | (PackedSSSMFP12BitsB << 20);
			}
			else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSPROFILE)
			{
				const uint PackedDiffuse20Bits = PackR7G7B6Gamma2(DiffuseAlbedo, Dither);
				const uint PackedF020Bits = PackR7G7B6Gamma2(F0, Dither);

				FSubstrateSubsurfaceHeader SSSDataHeader;
				SSSDataHeader.Bytes = MaterialTextureArrayUAV[uint3(PixelCoord, FirstSliceStoringSubstrateSSSData + 0)];

				float ProfileScl= SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSDataHeader) * DBufferData.OneMinusCoverage_BaseColor;
				uint  ProfileId = SubstrateSubSurfaceHeaderGetProfileId(SSSDataHeader);
				SubstrateSubSurfaceHeaderSetProfile(SSSDataHeader, ProfileScl, DBufferData.OneMinusCoverage_BaseColor == 0.0f ? 0 : ProfileId);
				// Note We cannot change the type using SubstrateSubSurfaceHeaderSetSSSType(SSSDataHeader, SSS_TYPE_NONE);
				// Because we are stuck using the SINGLE_OPTLEGACYMODE_SSSPROFILE mode.
				// So some SSSProfile material with decal will look difference when using deferred DBuffer instead of in shader application.

				MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = (MaterialTextureArrayUAV[uint3(PixelCoord, 0)] & 0x00FFFFFF) | (PackR8(Roughness) << 24);
				MaterialTextureArrayUAV[uint3(PixelCoord, 1)] = PackedDiffuse20Bits  | (PackR8(ProfileScl) << 24);
				MaterialTextureArrayUAV[uint3(PixelCoord, 2)] = (MaterialTextureArrayUAV[uint3(PixelCoord, 2)] & 0xFF000000) | PackedF020Bits;
				
				SubstrateStoreSubsurfaceHeader(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSDataHeader.Bytes);
				if (SubstrateSubSurfaceHeaderHasExtras(SSSDataHeader))
				{
					FSubstrateSubsurfaceExtras SSSDataExtras;
					SubstrateSubsurfaceExtrasSetBaseColor(SSSDataExtras, DiffuseAlbedo);
					SubstrateStoreSubsurfaceExtras(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSDataExtras.Bytes);
				}
			}
		}
		#endif // !SUBSTRATE_FASTPATH
	}
}
#endif // DBufferPassCS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#ifdef DBufferPassPS

// @param BufferUV - UV space in the DBuffer textures
void DBufferPassPS(
	float4 SVPos : SV_POSITION,
	out float4 OutGBufferA : SV_Target0,
	out float4 OutGBufferB : SV_Target1,
	out float4 OutGBufferC : SV_Target2)
{
	const uint2 PixelCoord = SVPos.xy;
	const bool bIsValid = all(PixelCoord < uint2(View.ViewSizeAndInvSize.xy));

	if (!bIsValid)
	{
		return;
	}

	#if 1
	uint ValidDBufferTargetMask = 0x07;
	#else
	// Early out of no decal has been written or if the pixel has no 'decal responseness'
	const uint ValidDBufferTargetMask = GetDBufferResponseMask(PixelCoord);
	BRANCH
	if (ValidDBufferTargetMask == 0)
	{
		return;
	}
	#endif

	const float2 BufferUV = SvPositionToBufferUV(float4(PixelCoord + 0.5f, 0, 0));

	// Setup default values, which mean that no decals are present in DBuffer
	float4 DBufferA = float4(0.0, 0.0, 0.0, 1.0);
	float4 DBufferB = float4(128.0f / 255.0f, 128.f / 255.5f, 128.f / 255.5f, 1.0);
	float4 DBufferC = float4(0.0, 0.0, 0.0, 1.0);
	{
		BRANCH
		if (ValidDBufferTargetMask & 0x1) { DBufferA = Texture2DSampleLevel(OpaqueBasePassDBufferATexture, OpaqueBasePassDBufferATextureSampler, BufferUV, 0);	}
		BRANCH
		if (ValidDBufferTargetMask & 0x2) { DBufferB = Texture2DSampleLevel(OpaqueBasePassDBufferBTexture, OpaqueBasePassDBufferBTextureSampler, BufferUV, 0);	}
		BRANCH
		if (ValidDBufferTargetMask & 0x4) { DBufferC = Texture2DSampleLevel(OpaqueBasePassDBufferCTexture, OpaqueBasePassDBufferCTextureSampler, BufferUV, 0);	}
	}

	OutGBufferA = DBufferB;
	OutGBufferB = DBufferC;
	OutGBufferC = DBufferA;
}
#endif // DBufferPassPS
