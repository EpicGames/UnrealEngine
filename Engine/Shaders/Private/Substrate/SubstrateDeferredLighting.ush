// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../AreaLightCommon.ush"

#if USE_LIGHT_FUNCTION_ATLAS
#include "../LightFunctionAtlas/LightFunctionAtlasCommon.usf"
#endif

// Sanity guard. 
#ifndef SUBSTRATE_ENABLED
	#define SUBSTRATE_ENABLED 1
	#error SUBSTRATE_ENABLED needs to be defined
#endif

// * If enabled, all material data will be loaded from the material buffer
// * If disabled, material data will be passed to SubstrateDeferredLighting as args. This is only support for single slab material
#ifndef SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
#define SUBSTRATE_LOAD_FROM_MATERIALCONTAINER 1
#endif

#if SUBSTRATE_ENABLED

float4 SubstrateReadPrecomputedShadowFactors(FSubstratePixelHeader SubstratePixelHeader, int2 PixelPos, Texture2D PrecomputedShadowTexture)
{
	if (SubstratePixelHeader.HasPrecShadowMask())
	{
	#if ALLOW_STATIC_LIGHTING
		float4 GBufferE = PrecomputedShadowTexture.Load(int3(PixelPos, 0));
	#else
		float4 GBufferE = 1;
	#endif
		return GBufferE;
	}
	return SubstratePixelHeader.HasZeroPrecShadowMask() ? 0.0f : 1.0f;
}

struct FSubstrateShadowTermInputParameters
{
	bool bEvaluateShadowTerm;
	float SceneDepth;
	float4 PrecomputedShadowFactors;
	float3 TranslatedWorldPosition;
	float4 LightAttenuation;
	float Dither;
	bool bUseShadowTermHairTransmittance;
};

FSubstrateShadowTermInputParameters GetInitialisedSubstrateShadowTermInputParameters()
{
	FSubstrateShadowTermInputParameters Params = (FSubstrateShadowTermInputParameters)0;
	return Params;
}

// Analytical lighting evaluation for Substrate material.
// * SUBSTRATE_LOAD_FROM_MATERIALCONTAINER == 1 : Unpack BSDF on-the-fly
// * SUBSTRATE_LOAD_FROM_MATERIALCONTAINER == 0 : BSDF passed as argument
FSubstrateDeferredLighting SubstrateDeferredLighting(
	FDeferredLightData LightData,
	float3 V,
	float3 L,
	float3 ToLight,
	float LightMask,
	FSubstrateShadowTermInputParameters SubstrateShadowTermInputParameters,
#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	FSubstrateMaterialContainer MaterialBuffer,
	FSubstrateAddressing SubstrateAddressing,
	FSubstratePixelHeader SubstratePixelHeader
#else
	uint2 PixelCoords,
	FSubstrateBSDF BSDF,
	float3x3 BSDFTangentBasis,
	float MaterialAO
#endif
	)
{
	FSubstrateDeferredLighting SubstrateLighting = GetInitialisedSubstrateDeferredLighting();
#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);
	Settings.bUseShadowTermHairTransmittance = SubstrateShadowTermInputParameters.bUseShadowTermHairTransmittance;
#else
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();
	Settings.bUseShadowTermHairTransmittance = SubstrateShadowTermInputParameters.bUseShadowTermHairTransmittance;
#endif

	FRectTexture RectTexture = ConvertToRectTexture(LightData);

	// Get the basic shadow terms
	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	const float MaterialAO = SubstrateGetAO(SubstratePixelHeader);
	#endif
	FShadowTerms ShadowTerms = { MaterialAO, 1.0, 1.0, InitHairTransmittanceData() };
	if (SubstrateShadowTermInputParameters.bEvaluateShadowTerm)
	{
		// Get the shadow term that is independent from Substrate closures.
		GetShadowTermsBase(
			SubstrateShadowTermInputParameters.SceneDepth,
			SubstrateShadowTermInputParameters.PrecomputedShadowFactors,
			LightData,
			SubstrateShadowTermInputParameters.LightAttenuation,
			ShadowTerms);

		SubstrateLighting.EstimatedCost += 0.3;	// add the cost of getting the shadow terms
	}
	
#if USE_LIGHT_FUNCTION_ATLAS
	const FLightFunctionColor LightFunctionColor = GetLocalLightFunctionCommon(SubstrateShadowTermInputParameters.TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
#endif

	#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
	Substrate_for (uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
	#endif
	{
		// Unpack BSDF data
		#if SUBSTRATE_LOAD_FROM_MATERIALCONTAINER
		FSubstrateBSDF BSDF = UnpackSubstrateBSDF(MaterialBuffer, SubstrateAddressing, SubstratePixelHeader);
		FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V, L);
		#else
		FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(BSDF, BSDFTangentBasis, V, L, PixelCoords);
		#endif

		FShadowTerms BSDFShadowTerms = ShadowTerms;
		if (SubstrateShadowTermInputParameters.bEvaluateShadowTerm)	// This is going to be a evaluation done at compile time
		{
			const float BSDFContactShadowOpacity= 1.f - abs(SLAB_SSSPHASEANISOTROPY(BSDF));
			const uint  BSDFShadingModelID		= SubstrateGetLegacyShadingModels(BSDF);

			// Now combine with shadow term that are dependent on the Substrate closures (screens space contact shadow) if necessary.
			ApplyContactShadowWithShadowTerms(
				SubstrateShadowTermInputParameters.SceneDepth,
				BSDFShadingModelID,
				BSDFContactShadowOpacity,
				LightData,
				SubstrateShadowTermInputParameters.TranslatedWorldPosition,
				L,
				SubstrateShadowTermInputParameters.Dither,
				BSDFShadowTerms);
		}

		// Lighting early out based on shadow attenuation
		const bool bNeedLighting = (BSDFShadowTerms.SurfaceShadow + BSDFShadowTerms.TransmissionShadow) > 0;

		#if !COMPILER_FXC // Work around "error X3524: duplicate attribute branch" in cases where this branch is constant
		BRANCH
		#endif
		if (bNeedLighting)
		{
			float Roughness = SubstrateGetBSDFRoughness(BSDFContext.BSDF);

			FAreaLight AreaLight = InitAreaLight();
			FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
			if (LightData.bRectLight)
			{
				FRect Rect = GetRect(ToLight, LightData);
				if (!IsRectVisible(Rect))
				{
					return GetInitialisedSubstrateDeferredLighting(); // Rect light can be non visible due to barn door occlusion
				}
				AreaLight = CreateAreaLight(Roughness, BSDFContext.N, BSDFContext.V, Rect, RectTexture);

				// We must have the SubstrateIntegrateBSDF inside the if due to the rectlight texture: it must be non ambiguous which texture is going to be used.
				// After the compilation, a local resource must map to a unique global resource (the default or the actual rect light texture).
				BSDFEvaluate = SubstrateEvaluateBSDFCommon(BSDFContext, BSDFShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_RECT);
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
				AreaLight = CreateAreaLight(Roughness, BSDFContext.N, BSDFContext.V, Capsule, LightData.bInverseSquared);

				// Lighting early out based NoL. Only for SUBSTRATE_FASTPATH, as this path as no SSS/Wrap features.
				#if SUBSTRATE_FASTPATH
				BRANCH
				if (AreaLight.NoL<=0)
				{
					return GetInitialisedSubstrateDeferredLighting(); // Rect light can be non visible due to barn door occlusion
				}
				#endif

				BRANCH
				if(IsAreaLight(AreaLight))
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(BSDFContext, BSDFShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_CAPSULE);
				}
				else
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(BSDFContext, BSDFShadowTerms, AreaLight, Settings, INTEGRATION_PUNCTUAL_LIGHT);
				}
			}

			SubstrateLighting.EstimatedCost += 0.4;	// add the cost of the lighting computations (should sum up to 1 form one light)

			float3 DiffuseLuminance = BSDFEvaluate.IntegratedDiffuseValue * LightData.DiffuseScale;
			float3 SpecularLuminance = BSDFEvaluate.IntegratedSpecularValue * LightData.SpecularScale;

			const float3 CommonMultiplier = LightData.Color * LightMask * LuminanceWeight(BSDFContext, BSDF)
		#if USE_LIGHT_FUNCTION_ATLAS
				* LightFunctionColor
		#endif
				;

		#if SUBSTRATE_OUTPUT_TOON_DATA
			const bool bIsToon = BSDF_GETTYPE(BSDFContext.BSDF) == SUBSTRATE_BSDF_TYPE_TOON;
			if (bIsToon)
			{
				const float3 ToonDiffuseLuminance = DiffuseLuminance * CommonMultiplier;

				SubstrateLighting.ToonData.DiffuseLuminance = ToonDiffuseLuminance;											// The diffuse illuminance to apply the diffuse ramp onto later.
				SubstrateLighting.ToonData.Weight			= BSDFEvaluate.ToonData.Weight * Luminance(CommonMultiplier);	// Accumulate more importance weight for current light (e.g. mask, intensity)
				SubstrateLighting.ToonData.ShadowNoL		= BSDFEvaluate.ToonData.ShadowNoL;								// Forward NoL.

				DiffuseLuminance = 0.0f;
			}
		#endif

			FLightAccumulator Out = (FLightAccumulator)0;
			LightAccumulator_AddSplit(Out, DiffuseLuminance, SpecularLuminance, DiffuseLuminance, CommonMultiplier, BSDFEvaluate.bPostProcessSubsurface);
			AccumulateSubstrateDeferredLighting(SubstrateLighting, Out, BSDFEvaluate.bPostProcessSubsurface, BSDF_GETISTOPLAYER(BSDF));

			SubstrateLighting.TotalDiffuseLighting += Out.TotalLightDiffuse;
			SubstrateLighting.TotalSpecularLighting += Out.TotalLightSpecular;
		}
	}

	return SubstrateLighting;
}

#endif // SUBSTRATE_ENABLED