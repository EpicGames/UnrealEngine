// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Sanity guard.  
#ifndef SUBSTRATE_ENABLED
	#define SUBSTRATE_ENABLED 1
	#error SUBSTRATE_ENABLED needs to be defined
#endif

#if SUBSTRATE_ENABLED

#include "Substrate.ush"

#ifndef SUBSTRATE_GLINTS_ALLOWED
#define SUBSTRATE_GLINTS_ALLOWED 1
#endif

#ifndef SUBSTRATE_GLINTS_IS
#define SUBSTRATE_GLINTS_IS 0 
#endif

#if SUBSTRATE_COMPLEXSPECIALPATH
#include "Glint/GlintThirdParty.ush"

#if SUBSTRATE_GLINTS_IS
#include "../BlueNoise.ush"
#endif 
#endif

#include "../ParticipatingMediaCommon.ush"
#include "../MonteCarlo.ush"
#include "../SHCommon.ush"
#include "../ShadingModels.ush"
#include "../ShadingModelsSampling.ush"
#include "../AreaLightCommon.ush"
#include "../ToonProfileCommon.ush"

#include "../HairStrands/HairStrandsCommon.ush"
#include "../HairStrands/HairStrandsDeepTransmittanceCommon.ush"
#include "../HairStrands/HairStrandsDeepTransmittanceDualScattering.ush"
#include "../HairStrands/HairStrandsEnvironmentLightingCommon.ush"

#ifndef SUBSTRATE_TRANSLUCENT_ENABLED
#define SUBSTRATE_TRANSLUCENT_ENABLED 0
#endif

#ifndef MATERIALBLENDING_ANY_TRANSLUCENT
#define MATERIALBLENDING_ANY_TRANSLUCENT 0
#endif

#ifndef SUBSTRATE_SSS_TRANSMISSION
#define SUBSTRATE_SSS_TRANSMISSION 0
#endif

#ifndef SUBSTRATE_DIFFUSE_COLOR_BOOST
#define SUBSTRATE_DIFFUSE_COLOR_BOOST 0
#endif

#ifndef SUBSTRATE_LIGHTPASS_APPLIES_TOON_DIFFUSE_RAMP
#define SUBSTRATE_LIGHTPASS_APPLIES_TOON_DIFFUSE_RAMP 1 
#endif

#ifndef SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL
#define SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL 0
#endif

#define SUBSTRATE_SUPPORTS_SIMPLEVOLUME (MATERIAL_IS_SUBSTRATE || SUBSTRATE_GBUFFER_FORMAT==1)

///////////////////////////////////////////////////////////////////////////////
// Substrate Macros for reducing permutation boiler plate
// 
// This allows to write a loop over all BSDFs as follow and support permutations
//
// Substrate_for(uint ClosureIndex = 0, ClosureIndex < SubstratePixelHeader.ClosureCount, ++ClosureIndex)
// {
//		FSubstrateBSDF BSDF = UnpackSubstrateBSDF(MaterialBuffer, SubstrateAddressing, SubstratePixelHeader);
//		...
// }

#if (SUBSTRATE_FASTPATH == 0 && SUBSTRATE_SINGLEPATH == 0 && SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL == 0 && SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1)
	#if COMPILER_FXC
		#define Substrate_for(X,Y,Z) [loop] for(X;Y;Z)
	#else
		#define Substrate_for(X,Y,Z) for(X;Y;Z)
	#endif
#else
	#define Substrate_for(X,Y,Z) X; if(Y)
#endif

#if SUBSTRATE_FASTPATH
	#define UnpackSubstrateBSDF(X, Y, Z) UnpackFastPathSubstrateBSDFIn(X, Y, Z)
#else
	#define UnpackSubstrateBSDF(X, Y, Z) UnpackSubstrateBSDFIn(X, Y, Z)
#endif

// SUBSTRATE_TODO put in a common file
// Point lobe in off-specular peak direction
float3 SubstrateGetOffSpecularPeakReflectionDir(float3 Normal, float3 ReflectionVector, float Roughness)
{
	float a = Square(Roughness);
	return lerp(Normal, ReflectionVector, (1 - a) * (sqrt(1 - a) + a));
}

///////////////////////////////////////////////////////////////////////////////
// BSDF evaluate and sampling

struct FSubstrateBSDFContext
{
	FSubstrateBSDF BSDF;

	float3 N;
	float3 X;
	float3 Y;
	float3 V;
	float3 R;
	float3 H;
	float3 B;

	float3 L;	// There to initialise the BxDFContext. Only used by SubstrateEvaluateBSDF, not by SubstrateImportanceSampleBSDF or SubstrateEvaluateForEnvLight

	BxDFContext Context;
	float SatNoL;
	float SatNoV;

	float3x3 TangentBasis;
	float3 TangentV;
	float3 TangentH;
	float3 TangentB; 
	float3 TangentL;
	
	uint2 PixelCoord; 

	void SubstrateUpdateBSDFContext(float3 NewL);
};

FSubstrateBSDFContext SubstrateCreateBSDFContext(float3x3 TangentBasis, FSubstrateBSDF BSDF, float3 V, float3 L, bool bHasValidL=true, uint2 InPixelCoord=0)
{
	FSubstrateBSDFContext BSDFContext = (FSubstrateBSDFContext)0;

	BSDFContext.BSDF = BSDF;

	BSDFContext.X = TangentBasis[0]; 
	BSDFContext.Y = TangentBasis[1];
	BSDFContext.N = TangentBasis[2]; 
	BSDFContext.V = V;
	BSDFContext.R = 2 * dot(BSDFContext.V, BSDFContext.N) * BSDFContext.N - BSDFContext.V;
	BSDFContext.L = bHasValidL ? L : BSDFContext.R;
	BSDFContext.H = normalize(BSDFContext.V + BSDFContext.L);
	BSDFContext.B = normalize(BSDFContext.R + BSDFContext.L);

	BSDFContext.Context = (BxDFContext)0;
#if SUBSTRATE_COMPLEXPATH
	if (BSDF_GETHASANISOTROPY(BSDF) != 0)
	{
		Init(BSDFContext.Context, BSDFContext.N, BSDFContext.X, BSDFContext.Y, BSDFContext.V, BSDFContext.L);
	}
	else
#endif
	{
		Init(BSDFContext.Context, BSDFContext.N, BSDFContext.V, BSDFContext.L);
	}
	BSDFContext.TangentBasis = float3x3(BSDFContext.X, BSDFContext.Y, BSDFContext.N);

	BSDFContext.TangentV = mul(BSDFContext.TangentBasis, BSDFContext.V);
	BSDFContext.TangentH = mul(BSDFContext.TangentBasis, BSDFContext.H);
	BSDFContext.TangentB = mul(BSDFContext.TangentBasis, BSDFContext.B);
	BSDFContext.TangentL = normalize(mul(BSDFContext.TangentBasis, BSDFContext.L));

	BSDFContext.SatNoL = saturate(BSDFContext.Context.NoL);
	BSDFContext.SatNoV = saturate(BSDFContext.Context.NoV);

#if SUBSTRATE_NORMAL_QUALITY==1
	// It looks like when we use the high quality normals, NoH can become 1.
	// This can cause D_GGX to be infinite du to a division by 0 when roughness is also 0.
	// So to avoid fireflies, we clamp NoH to a value a little bit below 1.
	BSDFContext.Context.NoH = clamp(BSDFContext.Context.NoH, 0.0f, 0.9999f);
#endif

	BSDFContext.PixelCoord = InPixelCoord;

	return BSDFContext;
}

FSubstrateBSDFContext SubstrateCreateBSDFContext(FSubstratePixelHeader SubstratePixelHeader, FSubstrateBSDF BSDF, const FSubstrateAddressing SubstrateAddressing, float3 V)
{
	float3 UnusedL = float3(0, 0, 1);
	float3x3 TangentBasis = SubstrateGetBSDFSharedBasis(SubstratePixelHeader, BSDF, SubstrateAddressing);
	return SubstrateCreateBSDFContext(TangentBasis, BSDF, V, UnusedL, false, SubstrateAddressing.PixelCoords);
}

FSubstrateBSDFContext SubstrateCreateBSDFContext(FSubstratePixelHeader SubstratePixelHeader, FSubstrateBSDF BSDF, const FSubstrateAddressing SubstrateAddressing, float3 V, float3 L)
{
	float3x3 TangentBasis = SubstrateGetBSDFSharedBasis(SubstratePixelHeader, BSDF, SubstrateAddressing);
	return SubstrateCreateBSDFContext(TangentBasis, BSDF, V, L, true, SubstrateAddressing.PixelCoords);
}

FSubstrateBSDFContext SubstrateCreateBSDFContext(FSubstrateBSDF BSDF, const float3x3 TangentBasis, float3 V, float3 L, uint2 PixelCoords)
{
	return SubstrateCreateBSDFContext(TangentBasis, BSDF, V, L, true, PixelCoords);
}

FSubstrateBSDFContext SubstrateCreateBSDFContext(FSubstrateBSDF BSDF, const float3x3 TangentBasis, float3 V, float3 L)
{
	return SubstrateCreateBSDFContext(TangentBasis, BSDF, V, L, true, 0/*SubstrateAddressing.PixelCoords*/);
}

void FSubstrateBSDFContext::SubstrateUpdateBSDFContext(float3 NewL)
{
	// Update all the data related to L
	this.L = NewL;
	Init(this.Context, this.N, this.V, this.L);
	this.R		 = 2 * dot(this.V, this.N) * this.N - this.V;
	this.H		 = normalize(this.V + this.L);
	this.B		 = normalize(this.R + this.L);

	this.TangentH = mul(this.TangentBasis, this.H);
	this.TangentB = mul(this.TangentBasis, this.B);
	this.TangentL = normalize(mul(this.TangentBasis, this.L));

	this.SatNoL	 = saturate(this.Context.NoL);
	this.SatNoV	 = saturate(this.Context.NoV);
}

///////////////////////////////////////////////////////////////////////////////
// Helper functions

float3 LuminanceWeight(in float SatNoL, in FSubstrateBSDF InBSDF)
{
#if SUBSTRATE_COMPLEXPATH
	// LuminanceWeight is the absorption from the shading point towards the surface 
	// in the normal direction. To compute the absorption towards the light, we 
	// reweight it with new distance L'
	// TransmittanceAboveAlongN = exp(sigma * T)  at normal incidence and for a thickness T. 
	// TransmittanceAboveAlongL	= exp(sigma * L)  with L= T . 1/NoL
	//							= exp(sigma * T . 1/NoL)
	//							= pow(exp(sigma * T), 1/NoL)
	// TransmittanceAboveAlongL	= pow(TransmittanceAboveAlongN, 1/NoL)
	const float DistanceL = rcp(max(SatNoL, 0.001f));
	const float3 TransmittanceAboveAlongL = any(InBSDF.TransmittanceAboveAlongN < 1.f) ? pow(max(InBSDF.TransmittanceAboveAlongN, 0.0001f), DistanceL) : InBSDF.TransmittanceAboveAlongN;
	return InBSDF.LuminanceWeightV * lerp(1.0f, TransmittanceAboveAlongL, InBSDF.CoverageAboveAlongN);
#else
	return InBSDF.LuminanceWeightV;
#endif
}

float3 LuminanceWeight(in FSubstrateBSDFContext InContext, in FSubstrateBSDF InBSDF)
{
	float SatNoL = InContext.SatNoL;

#if SUBSTRATE_COMPLEXPATH
	if (BSDF_GETTYPE(InBSDF) == SUBSTRATE_BSDF_TYPE_SLAB && !BSDF_GETISTOPLAYER(InBSDF))
	{
		// We do not have any information about the layers above at this stage so we use a default IOR mapping to water behavior (for all fo the layers).
		const float AirIOR = 1.0f;
		const float LayersAboveIOR = 1.5f;
		// Refract L coming toward the surface from air to matter according to IOR, and assuming all surface above have the same IOR.
		// Refraction also fades out as a function of the coverage of the surfaces above.
		float3 RefractedL = refract(-InContext.L, InContext.N, lerp(1.0f, AirIOR / LayersAboveIOR, InBSDF.CoverageAboveAlongN));
		SatNoL = saturate(dot(InContext.N, -RefractedL));
	}
#endif

	return LuminanceWeight(SatNoL, InBSDF);
}

float Substrate_D_GGX(half Roughness, half a2, half NoH)
{
	return D_GGX(a2, NoH);
}

half Substrate_D_GGX_Aniso(float ax, float ay, float NoH, float XoH, float YoH)
{
	return D_GGXaniso(ax, ay, NoH, XoH, YoH);
}

half Substrate_Vis_GGX(half Roughness, half a2, half NoV, half NoL)
{
#if SUBSTRATE_SHADING_QUALITY == 0
	return Vis_SmithJointApprox(a2, NoV, NoL);
#else
	return Vis_SmithJoint(a2, NoV, NoL);
#endif
}

half Substrate_Vis_GGX_Aniso(float ax, float ay, float NoV, float NoL, float XoV, float XoL, float YoV, float YoL)
{
	// SUBSTRATE_TODO: build approximation for the aniso version
	// Saturate NoV and NoL to avoid having negative value.
	return Vis_SmithJointAniso(ax, ay, saturate(NoV), saturate(NoL), XoV, XoL, YoV, YoL);
}

float3 Substrate_F_GGX(half3 F0, half3 F90, half VoH)
{
	return F_Schlick(F0, F90, VoH);
}

// Transition from Glint to GGX (0=Glint <-> 1=GGX)
float SubstrateTransitionGlintToGGX(float GlintValue)
{
#if SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED
	return saturate(saturate(GlintValue - 0.95f) / 0.05f);
#else
	return 1.f;
#endif
}

float Substrate_D_Glint(FSubstrateBSDFContext BSDFContext, float SafeRoughness, float GGX_D)
{
	float OutD = GGX_D;

	#if SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED
	BRANCH
	if (BSDF_GETHASGLINT(BSDFContext.BSDF))
	{
		FBeckmannDesc Beckmann = GGXToBeckmann(SafeRoughness);
		const float Glint_D = f_P(BSDFContext.TangentV, BSDFContext.TangentL, float3(Beckmann.Sigma.xx, Beckmann.Rho), SLAB_GLINT_VALUE(BSDFContext.BSDF), SLAB_GLINT_UV(BSDFContext.BSDF), SLAB_GLINT_UVDDX(BSDFContext.BSDF), SLAB_GLINT_UVDDY(BSDFContext.BSDF));

		// Transition from Glint to GGX
		OutD = lerp(Glint_D, GGX_D, SubstrateTransitionGlintToGGX(SLAB_GLINT_VALUE(BSDFContext.BSDF)));
	}
	#endif // SUBSTRATE_COMPLEXPATH

	return OutD;
}

float EvaluateGlintRect(FSubstrateBSDFContext BSDFContext, float SafeRoughness, float3 MeanLightWorldDirection)
{
	float Out = 1.0f;

	#if SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED
	BRANCH
	if (BSDF_GETHASGLINT(BSDFContext.BSDF))
	{
		float3 MeanLightLocalDirection = normalize(mul(BSDFContext.TangentBasis, MeanLightWorldDirection));
		FBeckmannDesc Beckmann = GGXToBeckmann(SafeRoughness);

		// It is wrong to multiply again the GGXevaluation together with the glint function (representing Beckmann D function).
		// But visually it works and that is an acceptable solution in the meantime we find something more correct.
		const float GlintSpec = f_P(BSDFContext.TangentV, MeanLightLocalDirection, float3(Beckmann.Sigma.xx, Beckmann.Rho),SLAB_GLINT_VALUE(BSDFContext.BSDF), SLAB_GLINT_UV(BSDFContext.BSDF), SLAB_GLINT_UVDDX(BSDFContext.BSDF), SLAB_GLINT_UVDDY(BSDFContext.BSDF));

		// Transition from Glint to GGX
		Out = lerp(GlintSpec, 1.f, SubstrateTransitionGlintToGGX(SLAB_GLINT_VALUE(BSDFContext.BSDF)));

	}
	#endif // SUBSTRATE_COMPLEXPATH

	return Out;
}

float EvaluateGlintEnv(FSubstrateBSDFContext BSDFContext, float SafeRoughness, inout float3 OutSpecularDirection)
{
	float Out = 1.0f;

#if SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED
	BRANCH
	if (BSDF_GETHASGLINT(BSDFContext.BSDF))
	{
		FBeckmannDesc Beckmann = GGXToBeckmann(SafeRoughness);
		const float TransitionGlintToGGX = SubstrateTransitionGlintToGGX(SLAB_GLINT_VALUE(BSDFContext.BSDF));
		float3 MeanLightLocalDirection = normalize(mul(BSDFContext.TangentBasis, OutSpecularDirection));

		// Importance sampling
		#if SUBSTRATE_GLINTS_IS
		{
			const float3 Random = float3(BlueNoiseVec2(BSDFContext.PixelCoord, View.StateFrameIndex), BlueNoiseScalar(BSDFContext.PixelCoord, View.StateFrameIndex));
					
			// Sample and refect view according the sampled m
			const float3 m_local = Sample_P(Random, BSDFContext.TangentV, float3(Beckmann.Sigma, Beckmann.Rho), \
							SLAB_GLINT_VALUE(BSDFContext.BSDF), SLAB_GLINT_UV(BSDFContext.BSDF), SLAB_GLINT_UVDDX(BSDFContext.BSDF), SLAB_GLINT_UVDDY(BSDFContext.BSDF));
			MeanLightLocalDirection =  normalize(2 * dot(BSDFContext.TangentV, m_local) * m_local - BSDFContext.TangentV);
			const float MeanLightWorldDirection = mul(transpose(BSDFContext.TangentBasis), MeanLightLocalDirection);
			OutSpecularDirection = normalize(lerp(OutSpecularDirection, MeanLightWorldDirection, TransitionGlintToGGX));
		}
		#endif

		// It is wrong to multiply again the split env lighting (for split-sum approxiation) together with the glint function (representing Beckmann D function).
		// But visually it works and that is an acceptable solution in the meantime we find something more correct.
		float GlintD = f_P(BSDFContext.TangentV, MeanLightLocalDirection, float3(Beckmann.Sigma, Beckmann.Rho), 
							SLAB_GLINT_VALUE(BSDFContext.BSDF), SLAB_GLINT_UV(BSDFContext.BSDF), SLAB_GLINT_UVDDX(BSDFContext.BSDF), SLAB_GLINT_UVDDY(BSDFContext.BSDF));
		
		// We need a way to lerp to the glint direction with a roughness of 0. 
		// SUBSTRATE_TODO recover glint direction.
		Out = lerp(saturate(GlintD), 1.f, TransitionGlintToGGX);
		// We keep the same SpecularSafeRoughness for single glint environment sampling to match the roughness of the surface overall, especially when the glints becomes infinitely small and thus invisible.
	}
#endif // SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED

	return Out;
}
///////////////////////////////////////////////////////////////////////////////////////////////////
// BSDF evaluation (punctual/area)

struct FSubstrateToonData
{
	float Weight;
	float ShadowNoL;
};

struct FSubstrateEvaluateResult
{
	float3 IntegratedDiffuseValue;
	float3 IntegratedSpecularValue;

	// Use for forward rendering using translucent lighting volume or per vertex lighting
	float3 DiffuseColor;

	float3 EmissivePathValue;

	float3 DiffusePathValue;
	float3 SpecularPathValue;
	float3 SpecularHazePathValue;
	float3 TransmissionPathValue;

	// The following probabilities are part of the pdf to work correctly with a monter carlo based integrator and as expected are not applied on the xxxPathValue.
	// So PathProbability should be applied on PathValue in the rasteriser to recover the correct balance.
	float SpecularPathProbability;
	float SpecularHazePathProbability;

	float  DiffusePDF;
	float  SpecularPDF;
	float  SpecularHazePDF;
	float  TransmissionPDF;

	float3 ThroughputV;					// Throughput to the next layer (from the view to the shading point). Contains many things such as transmittance or cloth energy conservation/preservation factors.
	float3 TransmittanceAlongN;			// Transmittance to the next layer along the surface normal. This is used later to compute the transmittance towards the light.
	bool   bPostProcessSubsurface;		// True if we need to separate the subsurface light contribution for the screen space diffusion process.
	bool   bApplyProjectedSolidAngle;	// True if the Saturate(NoL) factor should be applied or not over the value of the BSDF

	FSubstrateToonData  ToonData;
};

// PUNCTUAL corresponds to an evaluation (from in & out directions).
// When CAPSULE or RECT are used, special techniques are used to integrate specular lighting form those area light types.
#define INTEGRATION_PUNCTUAL_LIGHT		0
#define INTEGRATION_AREA_LIGHT_CAPSULE	1
#define INTEGRATION_AREA_LIGHT_RECT		2

void SubstrateSetupBSDFContext(inout FSubstrateBSDFContext BSDFContext, in bool bHasAnisotropy, inout float NoV, inout float VoH, inout float NoH, in float3 AreaLightL, in float SphereSinAlpha)
{
#if SUBSTRATE_COMPLEXPATH
	BRANCH
	if (bHasAnisotropy)
	{
		Init(BSDFContext.Context, BSDFContext.N, BSDFContext.X, BSDFContext.Y, BSDFContext.V, AreaLightL);

		NoV = BSDFContext.Context.NoV;
		VoH = BSDFContext.Context.VoH;
		NoH = BSDFContext.Context.NoH;
	}
	else
#endif
	{
		Init(BSDFContext.Context, BSDFContext.N, BSDFContext.V, AreaLightL);

		NoV = BSDFContext.Context.NoV;
		VoH = BSDFContext.Context.VoH;
		NoH = BSDFContext.Context.NoH;

		SphereMaxNoH(BSDFContext.Context, SphereSinAlpha, true);
	}
	BSDFContext.Context.NoV = saturate(max(abs(BSDFContext.Context.NoV), SUBSTRATE_EPSILON));
	BSDFContext.SatNoV = BSDFContext.Context.NoV;
	BSDFContext.SatNoL = saturate(BSDFContext.Context.NoL);
}

FSubstrateEvaluateResult SubstrateEvaluateBSDFCommon(FSubstrateBSDFContext BSDFContext, FShadowTerms ShadowTerms, FAreaLight AreaLight, FSubstrateIntegrationSettings Settings, int IntegrationType)
{
	FSubstrateEvaluateResult Sample = (FSubstrateEvaluateResult)0;

	const float OpaqueBSDFThroughput = 0.0f;

	const uint BSDFType = BSDF_GETTYPE(BSDFContext.BSDF);
	switch (BSDFType)
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			const bool bIsRectLight 	= IntegrationType == INTEGRATION_AREA_LIGHT_RECT;

			float3 DiffuseColor			= SLAB_DIFFUSEALBEDO(BSDFContext.BSDF);
			float3 F0					= SLAB_F0(BSDFContext.BSDF);
			float3 F90					= SLAB_F90(BSDFContext.BSDF);
			const float SafeRoughness	= MakeRoughnessSafe(SLAB_ROUGHNESS(BSDFContext.BSDF), SUBSTRATE_MIN_GGX_ROUGHNESS);
			const bool bHasAnisotropy	= BSDF_GETHASANISOTROPY(BSDFContext.BSDF) && !bIsRectLight && SUPPORTS_ANISOTROPIC_MATERIALS;
			const bool bHaziness		= BSDF_GETHASHAZINESS(BSDFContext.BSDF);

			// Specular occlusion is only used here once to affect the F90 source parameters. F0 will decrease naturally to 0.
			F90 *= F0RGBToMicroOcclusion(F0);

			if (Settings.bForceFullyRough)
			{
				// When rendering reflection captures, the BSDF roughness has already been forced to 1 using View.RoughnessOverrideParameter (see SubstrateSanitizeBSDF).
				EnvBRDFApproxFullyRough(DiffuseColor, F0, F90);
			}

		#if SUBSTRATE_DIFFUSE_COLOR_BOOST
			DiffuseColor = ApplyDiffuseColorBoost(DiffuseColor, LumenHardwareRayTracingUniformBuffer.DiffuseColorBoost);
		#endif

			// When using SimpleClearCoat with CLEAR_COAT_BOTTOM_NORMAL, the AreaLight is constructed with the clear coat's 
			// normal. In order to get the correct diffuse response, AreaLight_NoL and AreaLight_Falloff are recomputed with 
			// the bottom layer's normal.
			bool bHazeAsSimpleClearCoat = false;
			float AreaLight_NoL = AreaLight.NoL;
			float AreaLight_Falloff = AreaLight.Falloff;
		#if SUBSTRATE_FASTPATH==0 && CLEAR_COAT_BOTTOM_NORMAL 
			float  bUseBottomNormal = false;
			float3 TopNormal = BSDFContext.N;
			BRANCH
			if (BSDF_GETHASHAZINESS(BSDFContext.BSDF))
			{
				FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
				bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;
				BRANCH
				if (bHazeAsSimpleClearCoat && Haziness.HasBottomNormal())
				{
					float3 BottomNormal = SubstrateGetSimpleCoatBottomNormal(Haziness.BottomNormalOct, BSDFContext.N);
					bUseBottomNormal = true;

					// Reconstruct bottom layer's tangent basis
					BSDFContext.N = BottomNormal;
					BSDFContext.X = BSDFContext.X;
					BSDFContext.Y = normalize(cross(BSDFContext.N, BSDFContext.X));
					BSDFContext.TangentBasis[0] = BSDFContext.X;
					BSDFContext.TangentBasis[1] = BSDFContext.Y;
					BSDFContext.TangentBasis[2] = BottomNormal;

					// Recompute AreaLight_Falloff and AreaLight_NoL with the correct diffuse normal
					BRANCH
					if (bIsRectLight)
					{
						RectIrradianceLambert(BottomNormal, AreaLight.Rect, AreaLight_Falloff, AreaLight_NoL);
					}
					else
					{
						// For point light, AreaLight.SphereSinAlpha=0, and SphereHorizonCosWrap() becomes a NOP
						const float NoL = saturate(dot(BottomNormal, AreaLight.DiffuseL));
						AreaLight_NoL = SphereHorizonCosWrap(NoL, Square(AreaLight.SphereSinAlpha));
					}
				}
			}
		#endif

			float Alpha2Spec = Pow4(SafeRoughness);

			float NoV, VoH, NoH;
			SubstrateSetupBSDFContext(BSDFContext, bHasAnisotropy, NoV, VoH, NoH, AreaLight.SpecularL, AreaLight.SphereSinAlpha);

			////
			//// Evaluate the diffuse component.
			////
#if MATERIAL_ROUGHDIFFUSE
			if (Settings.bRoughDiffuseEnabled && any(DiffuseColor > 0))
			{
				// * If the specular layer is anisotropic, the diffuse takes the 'main' roughness value rather than the tangent/bitangent value
				// * The Chan diffuse model bakes transmittance specular directional albedo assuming F=0.04. In previous code we reapplied 
				//   this transmittance, as the energy preservation code we remove it as well. However the visual effect was small 
				//   and the cost was rather large (~8%). See the code below for where this is handled, this is no longer necessary
				//   with the latest model.
				Sample.DiffusePathValue = Diffuse_GGX_Rough(DiffuseColor, SafeRoughness, NoV, AreaLight_NoL, VoH, NoH, GetAreaLightDiffuseMicroReflWeight(AreaLight));
			}
			else
#endif
			{
				Sample.DiffusePathValue = Diffuse_Lambert(DiffuseColor);
			}
			Sample.IntegratedDiffuseValue += (ShadowTerms.SurfaceShadow * AreaLight_NoL * AreaLight_Falloff) * Sample.DiffusePathValue * AreaLight.FalloffColor;
			Sample.DiffuseColor		= DiffuseColor;
			Sample.DiffusePDF		= BSDFContext.SatNoL / PI;
			Sample.bPostProcessSubsurface = false;




			////
			//// Evaluate the specular component.
			//// This takes into account multiple scattering, micro occlusion.
			//// Note: anisotropy completely disables area integrations. Lights fall back to punctual.
			////

			// Primary highlight. For clear coat, this is the bottom layer.
			float PDF = 0;
			float DirectionalAlbedo_SpecularTransmission = 1.0f;
			{
				FBxDFEnergyType MSScale = 1.0f;
				{
					FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(SafeRoughness, BSDFContext.Context.NoV, F0, F90);
					DirectionalAlbedo_SpecularTransmission = ComputeEnergyPreservation(EnergyTerms);
					MSScale = ComputeEnergyConservation(EnergyTerms);
				}
			
				float D = 0;
				float Vis = 0;

				float3 RectLightSpec = 0;
				#if SUBSTRATE_COMPLEXPATH
				BRANCH
				if (bHasAnisotropy)
				{
					// Generalized microfacet specular
					{
						float  Alpha  = Square(SafeRoughness);
						float2 AlphaXY = 0;
						GetAnisotropicRoughness(Alpha, SLAB_ANISOTROPY(BSDFContext.BSDF), AlphaXY.x, AlphaXY.y);
				
						// GGX lobe
						{
							D = Substrate_D_GGX_Aniso(AlphaXY.x, AlphaXY.y, BSDFContext.Context.NoH, BSDFContext.Context.XoH, BSDFContext.Context.YoH);
						}

						// Glint shading only in the complex path
						{
							D = Substrate_D_Glint(BSDFContext, SafeRoughness, D);
						}

						Vis = Substrate_Vis_GGX_Aniso(AlphaXY.x, AlphaXY.y, BSDFContext.Context.NoV, AreaLight.NoL, BSDFContext.Context.XoV, BSDFContext.Context.XoL, BSDFContext.Context.YoV, BSDFContext.Context.YoL);
						const float H_PDF = VisibleGGXPDF_aniso(BSDFContext.TangentV, BSDFContext.TangentH, AlphaXY);
						PDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
					}
				}
				else
				#endif
				{
					BRANCH
					if (bIsRectLight)
					{
						// In this case, we set D to 1 and Vis will contain the area light / GGX integration
						{
							float3 MeanLightWorldDirection = 0.0f;

							// GGX lobe
							{
								RectLightSpec = RectGGXApproxLTC(SafeRoughness, F0, F90, BSDFContext.N, BSDFContext.V, AreaLight.Rect, AreaLight.Texture, MeanLightWorldDirection);
							}
							
							// Glint shading only in the complex path
							{
								RectLightSpec *= EvaluateGlintRect(BSDFContext, SafeRoughness, MeanLightWorldDirection);
							}

							const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Alpha2Spec);
							PDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
						}
					}
					else
					{
						// Special override for roughness for supporting area light integrator with Sphere/Tube/Disk light, which modifies/increase roughness.

						// Generalized microfacet specular
						{
							// The energy term needs to be computed before the D evaluation as this function modifies Alpha2Spec.
							const float Energy = EnergyNormalization(Alpha2Spec, BSDFContext.Context.VoH, AreaLight);

							// GGX lobe
							{
								D = Substrate_D_GGX(SafeRoughness, Alpha2Spec, BSDFContext.Context.NoH);
							}

							// Glint shading only in the complex path
							{
								D = Substrate_D_Glint(BSDFContext, SafeRoughness, D);
							}

							// Area light energy compensation
							{
								D *= Energy;
							}

							Vis = Substrate_Vis_GGX(SafeRoughness, Alpha2Spec, BSDFContext.Context.NoV, AreaLight.NoL);
							const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Alpha2Spec);
							PDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
						}
					}
				}

				if (bIsRectLight)
				{
					Sample.SpecularPathValue = MSScale * RectLightSpec;
				}
				else
				{
					const float3 FresnelTerm = Substrate_F_GGX(F0, F90, BSDFContext.Context.VoH);
					Sample.SpecularPathValue = D * Vis * MSScale * FresnelTerm;
				}

				Sample.SpecularPathProbability     = 1.0f;
				Sample.SpecularHazePathProbability = 0.0f;
				Sample.IntegratedSpecularValue = Sample.SpecularPathValue;
			}

			bool bHazeHasAnisotropy = bHasAnisotropy && !bHazeAsSimpleClearCoat; // Simple clear coat top do not receive anisotropy
		#if SUBSTRATE_FASTPATH==0 && CLEAR_COAT_BOTTOM_NORMAL 
			if (bUseBottomNormal)
			{
				BSDFContext.N = TopNormal;
				BSDFContext.TangentBasis[2] = TopNormal;
				SubstrateSetupBSDFContext(BSDFContext, bHazeHasAnisotropy, NoV, VoH, NoH, AreaLight.SpecularL, AreaLight.SphereSinAlpha);
			}
		#endif

			// Secondary highlight. For clear coat, this is the top layer.
			float HazePDF = 0;
			#if SUBSTRATE_FASTPATH==0
			BRANCH
			if (bHaziness)
			{
				// F0 / F90 have already been affected by specular micro occlusion.
				float3 HazeF0 = F0;
				float3 HazeF90= F90;
			
				const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
				const float HazeWeight = Haziness.Weight;
				const float HazeSafeRoughness = MakeRoughnessSafe(Haziness.Roughness, SUBSTRATE_MIN_GGX_ROUGHNESS);
				const bool bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;

				if (bHazeAsSimpleClearCoat)
				{
					HazeF0 = 0.04f;
					HazeF90 = 1.0f;
					// In this case, no specular micro occlusion happens.
				}

				FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(HazeSafeRoughness, BSDFContext.Context.NoV, HazeF0, HazeF90);
				FBxDFEnergyType HazeMSScale = ComputeEnergyConservation(EnergyTerms);
				DirectionalAlbedo_SpecularTransmission = lerp(DirectionalAlbedo_SpecularTransmission, ComputeEnergyPreservation(EnergyTerms), HazeWeight);

				float HazeD = 0;
				float HazeVis = 0;

				float3 RectLightSpecHaze = 0;
				#if SUBSTRATE_COMPLEXPATH
				BRANCH
				if (bHazeHasAnisotropy)
				{
					float2 HazeAlpha = 0;
					GetAnisotropicRoughness(HazeSafeRoughness, SLAB_ANISOTROPY(BSDFContext.BSDF), HazeAlpha.x, HazeAlpha.y);

					HazeD   = Substrate_D_GGX_Aniso(HazeAlpha.x, HazeAlpha.y, BSDFContext.Context.NoH, BSDFContext.Context.XoH, BSDFContext.Context.YoH);
					HazeVis = Substrate_Vis_GGX_Aniso(HazeAlpha.x, HazeAlpha.y, BSDFContext.Context.NoV, BSDFContext.Context.NoL, BSDFContext.Context.XoV, BSDFContext.Context.XoL, BSDFContext.Context.YoV, BSDFContext.Context.YoL);
					const float H_PDF = VisibleGGXPDF_aniso(BSDFContext.TangentV, BSDFContext.TangentH, HazeAlpha);
					HazePDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
				}
				else
				#endif
				{
					if (bIsRectLight)
					{
						{
							RectLightSpecHaze = RectGGXApproxLTC(HazeSafeRoughness, HazeF0, HazeF90, BSDFContext.N, BSDFContext.V, AreaLight.Rect, AreaLight.Texture);
					
							const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Pow4(HazeSafeRoughness));
							HazePDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
						}
					}
					else
					{
						// Special override for roughness for supporting area light integrator with Sphere/Tube/Disk light, which modifies/increase roughness.
						{
							float Alpha2SpecHaze = Pow4(HazeSafeRoughness);
							const float Energy = EnergyNormalization(Alpha2SpecHaze, BSDFContext.Context.VoH, AreaLight);
							HazeD = Substrate_D_GGX(HazeSafeRoughness, Alpha2SpecHaze, BSDFContext.Context.NoH) * Energy;
							HazeVis = Substrate_Vis_GGX(HazeSafeRoughness, Alpha2SpecHaze, BSDFContext.Context.NoV, AreaLight.NoL);
							const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Alpha2SpecHaze);
							HazePDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
						}
					}
				}

				const float3 HazeFresnelTerm = Substrate_F_GGX(HazeF0, HazeF90, BSDFContext.Context.VoH);
				if (bIsRectLight)
				{					
					Sample.SpecularHazePathValue = RectLightSpecHaze * HazeMSScale;
				}
				else
				{
					Sample.SpecularHazePathValue = HazeD * HazeVis * HazeMSScale * HazeFresnelTerm;
				}

				// Reminder: 
				//	- bHazeAsSimpleClearCoat == false means we have two specular lobes that are lerped.
				//	- bHazeAsSimpleClearCoat == true  means we have a bottom specular lobe (roughness and F0 user specified) and a top specular lob (hard coded absortion, F0=0.04 and user specified roughness.
				BRANCH
				if (bHazeAsSimpleClearCoat)
				{
					const BxDFContext ClearCoatContext = RefractClearCoatContext(BSDFContext.Context);
					const float SlabMetallic = F0RGBToMetallic(F0);
					const float3 HazeClearCoatTransmittance = SimpleClearCoatTransmittance(ClearCoatContext.NoL, ClearCoatContext.NoV, SlabMetallic, lerp(SLAB_DIFFUSEALBEDO(BSDFContext.BSDF), F0, SlabMetallic));
				
					const float TopLayerCoverage = HazeWeight;
					const float TopLayerSpecularTransmittionApprox = saturate(1.0f - HazeFresnelTerm.x); // Simple and do not use the LUT for now. With bHazeAsSimpleClearCoat, Fresnel is achromatic
					const float3 TopLayerThrouput = lerp(1.0f, HazeClearCoatTransmittance * TopLayerSpecularTransmittionApprox, TopLayerCoverage);

					const float TopLayerThrouputGrey = dot(TopLayerThrouput, (1.0 / 3.0).xxx);
					Sample.SpecularPathProbability = TopLayerThrouputGrey / (TopLayerCoverage + TopLayerThrouputGrey);
					Sample.SpecularHazePathProbability = TopLayerCoverage / (TopLayerCoverage + TopLayerThrouputGrey);

					// Clear coat applied over bottom layer diffuse component
					Sample.DiffusePathValue *= TopLayerThrouput;
					Sample.IntegratedDiffuseValue *= TopLayerThrouput;
					// And to the bottom layer specular affected by top layer throughput, on top of which we add the top layer specular
					Sample.IntegratedSpecularValue = Sample.SpecularPathValue * TopLayerThrouput + Sample.SpecularHazePathValue * TopLayerCoverage;
				}
				else
				{
					Sample.SpecularPathProbability     = (1.0f - HazeWeight);
					Sample.SpecularHazePathProbability = HazeWeight;
					Sample.IntegratedSpecularValue = lerp(Sample.SpecularPathValue, Sample.SpecularHazePathValue, HazeWeight);
				}
			}
			#endif

			// Specular to Diffuse energy preservation
			{
				#if ROUGH_DIFFUSE_BRDF_VERSION == 1 && MATERIAL_ROUGHDIFFUSE && SUBSTRATE_SHADING_QUALITY == 1	
				// We remove the Chan model baked transmittance specular directional albedo assuming F=0.04.
				// Then we apply the transmittance according to the current surface setup.
				// To make this a safe operation:
				//	- We clamp SpecularTChan to a threshold to avoid div by 0 when the material is a metal. 
				//	- When the material tends to be metal, DirectionalAlbedo_SpecularTransmission will naturally tend to be 0
				// Bypass this correction when SUBSTRATE_SHADING_QUALITY == 0 (i.e., med/low quality), for trading accuracy for speed
				// We also do not apply that correction when rendering SIMPLE_VOLUME slabs:
				//	- In this case there is no specular transmission baked in the participating media scattering model we use for coating layer.
				//	- And specular throughput needs to be applied on the surface transmittance too to be energy conservative.
				if (Settings.bRoughDiffuseEnabled && BSDF_GETSSSTYPE(BSDFContext.BSDF) != SSS_TYPE_SIMPLEVOLUME)
				{
					const float ChanF0 = 0.04;
					FBxDFEnergyTermsA EnergyTerms = ComputeGGXSpecEnergyTermsA(SafeRoughness, BSDFContext.Context.NoV, ChanF0, 1.0f);
					const float SpecularTransmitionChanF0ToMaterialF0 = DirectionalAlbedo_SpecularTransmission / max(SUBSTRATE_EPSILON, ComputeEnergyPreservation(EnergyTerms));
					DirectionalAlbedo_SpecularTransmission = SpecularTransmitionChanF0ToMaterialF0;
				}
				#endif

				// Apply energy conservation on the diffuse component
				// If the specular layer is anisotropic, the energy term is computed onto the 'main' roughness [Kulla 2019]
				//
				Sample.DiffusePathValue *= DirectionalAlbedo_SpecularTransmission;
				Sample.IntegratedDiffuseValue *= DirectionalAlbedo_SpecularTransmission;
			}

			// Lighting LUT
			#if SUBSTRATE_SPECPROFILE_ENABLED
			BRANCH
			if (BSDF_GETHASSPECPROFILE(BSDFContext.BSDF))
			{
				Sample.IntegratedSpecularValue *= EvaluateSpecularProfile(SLAB_SPECPROFILEID(BSDFContext.BSDF), BSDFContext.SatNoV, BSDFContext.SatNoL, BSDFContext.Context.VoH, BSDFContext.Context.NoH);
			}
			#endif

			{
				float3 CommonTerm = 0.0f;
				if (bIsRectLight)
				{
					CommonTerm = ShadowTerms.SurfaceShadow; /* AreaLight.NoL and falloff is part of the LTC integration already */
				}
				else
				{
					CommonTerm = (ShadowTerms.SurfaceShadow * AreaLight.NoL * AreaLight.Falloff) * AreaLight.FalloffColor;
				}
				Sample.IntegratedSpecularValue *= CommonTerm;
			}

			Sample.SpecularPDF = PDF * Sample.SpecularPathProbability;
			Sample.SpecularHazePDF = HazePDF * Sample.SpecularHazePathProbability;


			////
			//// Evaluate emissive and set the sample throughput (transmittance to next layer) corresponding to an opaque slab of matter.
			////
			
			// we do not need to add emissive for the BRDF TotalSpec or TotalDiff values as this is handled separately
			Sample.EmissivePathValue = BSDF_GETEMISSIVE(BSDFContext.BSDF);

			Sample.ThroughputV			= OpaqueBSDFThroughput;
			Sample.TransmittanceAlongN	= OpaqueBSDFThroughput;

			////
			//// Evaluate approximated SSS (using wrap lighting). Temp code. SUBSTRATE_TODO: unify evaluation
			////

			#if SUBSTRATE_FASTPATH==0
			if (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_WRAP || BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
			{
				// The wrap lighting is a non-physically based shading which intends to match legacy behavior.
				// For doing so the look MFP & phase function anisotropy are respectively reinterpreted as 'SubSurfaceColor' and 'Opacity', per legacy shading model
				const bool bTwoSided = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_TWO_SIDED_WRAP;
				const float TransmittanceNoL = 1.0f;
				const float3 SlabDiffuseColor = bTwoSided ? DiffuseColor : float3(1, 1, 1);
				const FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(SlabDiffuseColor, SLAB_SSSMFP(BSDFContext.BSDF));
				const float3 SubSurfaceColor = IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, TransmittanceNoL);
				const float Opacity = 1.f - abs(SLAB_SSSPHASEANISOTROPY(BSDFContext.BSDF));

				float3 TransmissionThroughput = 0;
				if (bTwoSided)
				{
					// Legacy Foliage
					
					// http://blog.stevemcauley.com/2011/12/03/energy-conserving-wrapped-diffuse/
					const float Wrap = 0.5;
					const float WrapNoL = saturate((-dot(BSDFContext.N, BSDFContext.L) + Wrap) / Square(1 + Wrap));
					// Scatter distribution
					const float VoL = dot(BSDFContext.V, BSDFContext.L);
					const float Scatter = Substrate_D_GGX(0.6, 0.6 * 0.6, saturate(-VoL));

					TransmissionThroughput = (WrapNoL * Scatter) * SubSurfaceColor;
				}
				else
				{
					// Legacy Subsurface
					 
					// To get an effect when you see through the material (hard coded pow constant)
					const half InScatter = pow(saturate(dot(AreaLight.DiffuseL, -BSDFContext.V)), 12) * lerp(3, .1f, Opacity);
					// Wrap around lighting, 
					// * /(PI*2) to be energy consistent (hack do get some view dependnt and light dependent effect)
					// * Opacity of 0 gives no normal dependent lighting, Opacity of 1 gives strong normal contribution	
					// * Simplified version (with w=.5,  n=1.5): 
					//     half WrappedDiffuse = 2 * pow(saturate((dot(N, L) + w) / (1.0f + w)), n) * (n + 1) / (2 * (1 + w));
					//     NormalContribution = WrappedDiffuse * Opacity + 1 - Opacity;
					const half WrappedDiffuse = pow(saturate(dot(BSDFContext.N, AreaLight.DiffuseL) * (1.f / 1.5f) + (0.5f / 1.5f)), 1.5f) * (2.5f / 1.5f);
					const half NormalContribution = lerp(1.f, WrappedDiffuse, Opacity);
					const half BackScatter = /* GBuffer.GBufferAO */ NormalContribution / (PI * 2);

					// Transmission
					// * Emulate Beer-Lambert absorption by retrieving extinction coefficient from SubSurfaceColor. Subsurface is interpreted as a 'transmittance color' 
					//   at a certain 'normalized' distance (SubSurfaceColorAsTransmittanceAtDistanceInMeters). This is a coarse approximation for getting hue-shiting. 
					// * TransmittedColor is computed for the 1-normalized distance, and then transformed back-and-forth in HSV space to preserving the luminance value 
					//   of the original color, but getting hue shifting
					const half3 ExtinctionCoefficients = TransmittanceToExtinction(SubSurfaceColor, View.SubSurfaceColorAsTransmittanceAtDistanceInMeters);
					const half3 RawTransmittedColor = ExtinctionToTransmittance(ExtinctionCoefficients, 1.0f /*At 1 meters, as we use normalized units*/);
					const half3 TransmittedColor = HSV_2_LinearRGB(half3(LinearRGB_2_HSV(RawTransmittedColor).xy, LinearRGB_2_HSV(SubSurfaceColor).z));

					// Lerp to never exceed 1 (energy conserving)
					const float SubsurfaceTransmittance = ShadowTerms.TransmittanceOrOpticalThickness;
					TransmissionThroughput = lerp(BackScatter, 1, InScatter) * lerp(TransmittedColor, SubSurfaceColor, SubsurfaceTransmittance);
				}

				Sample.TransmissionPDF = 1.0f / (4.0f * PI); // SUBSTRATE_TODO this currently match the uniform sphere sampling from SubstrateImportanceSampleBSDF
				Sample.TransmissionPathValue = TransmissionThroughput; // We are not applying DirectionalAlbedo_SpecularTransmission here to match legacy shading model for now.
				Sample.IntegratedDiffuseValue += (ShadowTerms.TransmissionShadow * AreaLight.Falloff) * AreaLight.FalloffColor * Sample.TransmissionPathValue;
			}
			#endif

			////
			//// Evaluate transmitted light through a mesh due to sub surface scattering.
			////
			#if SUBSTRATE_FASTPATH==0
			const bool bIsSubsurfaceDiffusionProfile = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION_PROFILE;
			const bool bIsSubsurfaceDiffusionMFP = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION;
			Sample.bPostProcessSubsurface = (bIsSubsurfaceDiffusionProfile || bIsSubsurfaceDiffusionMFP) && SUBSTRATE_SSS_MATERIAL_OVERRIDE;
			#if SUBSTRATE_SSS_TRANSMISSION
			if (bIsSubsurfaceDiffusionProfile || bIsSubsurfaceDiffusionMFP)
			{
				const float ShadowOpticalDepth = DecodeOpticalDepthFromShadowMask(ShadowTerms.TransmittanceOrOpticalThickness);

				// TODO: we end up with optical depth here but it seems SSS GetTransmissionProfile or GetBurleyTransmission expect Thickness in centimeters. A bad naming case?
				float ThicknessInCm = ShadowOpticalDepth;

				float3 TransmissionThroughput = 1.0f;
				float PhaseFunction = 0.0f;
				BRANCH
				if (bIsSubsurfaceDiffusionProfile)
				{
					ThicknessInCm = BSDF_GETISTHIN(BSDFContext.BSDF) ? SLAB_SSSPROFILETHICKNESSCM(BSDFContext.BSDF) : ThicknessInCm;

					const uint ProfileId = SLAB_SSSPROFILEID(BSDFContext.BSDF);
					const FTransmissionProfileParams TransmissionParams = GetTransmissionProfileParams(ProfileId);
					TransmissionThroughput	= GetTransmissionProfile(ProfileId, ThicknessInCm).rgb;

					const float PhaseFunctionAnisotropy = TransmissionParams.ScatteringDistribution;
					const float OneOverIOR				= TransmissionParams.OneOverIOR;
					const float3 RefracV				= refract(BSDFContext.V, -BSDFContext.N, OneOverIOR);
					// We must use ApproximateHG to match the legacy for the moment.
					// Later, we should look at using the same phase function as SSS_TYPE_DIFFUSION (Keep in mind that phase function should not be multiple with multiple scattering, only single scattering)
					PhaseFunction						= ApproximateHG(dot(-BSDFContext.L, RefracV), TransmissionParams.ScatteringDistribution);
				}
				else
				{
					// When the surface 'IsThin' MFP is rescaled to SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM
					const bool bIsThin = BSDF_GETISTHIN(BSDFContext.BSDF);
					ThicknessInCm = bIsThin ? SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM : ThicknessInCm;

					const float3 MeanFreePathInCm = SLAB_SSSMFP(BSDFContext.BSDF);
					const float3 SubsurfaceAlebdo = SLAB_DIFFUSEALBEDO(BSDFContext.BSDF);
					TransmissionThroughput = GetBurleyTransmission(SubsurfaceAlebdo, max(SUBSTRATE_EPSILON, MeanFreePathInCm), ThicknessInCm).xyz;

					const float PhaseFunctionAnisotropy = SLAB_SSSPHASEANISOTROPY(BSDFContext.BSDF);
					const float OneOverIOR				= 1.f / DielectricF0ToIor(F0.y);
					const float3 RefracV				= refract(BSDFContext.V, -BSDFContext.N, OneOverIOR);
					PhaseFunction						= HenyeyGreensteinPhase(PhaseFunctionAnisotropy, dot(BSDFContext.L, RefracV));
				}

				Sample.ThroughputV				= OpaqueBSDFThroughput; // SSS is not translucent as of today
				Sample.TransmittanceAlongN		= OpaqueBSDFThroughput; // idem
				Sample.TransmissionPathValue	= TransmissionThroughput * PhaseFunction;
				Sample.TransmissionPDF			= 1.0f / (4.0f * PI);		// SUBSTRATE_TODO this currently match the uniform sphere sampling from SubstrateImportanceSampleBSDF

				Sample.IntegratedDiffuseValue += (ShadowTerms.TransmissionShadow * AreaLight.Falloff) * AreaLight.FalloffColor * Sample.TransmissionPathValue;
			}
			#endif
			#endif

			////
			//// Evaluate a layer of participating media: scattering and transmittance.
			//// This is used for optically thin translucent objects, or non-bottom layer of a material.
			////

			#if ((SUBSTRATE_FASTPATH==0 && SUBSTRATE_SUPPORTS_SIMPLEVOLUME) || MATERIALBLENDING_ANY_TRANSLUCENT) // !Fast path or rendering translucent materials
			BRANCH
			if (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_SIMPLEVOLUME)
			{
				FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(DiffuseColor, SLAB_SSSMFP(BSDFContext.BSDF));
				const float DiffuseToVolumeBlend = SubstrateSlabDiffuseToVolumeBlend(PM);

				// !!! WARNING !!!
				// SlabDirectionalAlbedo represents single+multiple scattering and has been evaluated for an isotropic phase function.
				// However want to have some direcitonality now simulated for our medium.
				// To this aim, we compensate for the uniform phase function and apply a HG phase function.
				// This is wrong because ontly single scattering should be directly affected by a phase function this way. 
				// We still have decided to go with such a solution to get artist authroble directional lighting (while we are looking for something better)
				const float CosTheta		= BSDFContext.Context.VoL;
				const float HGPhaseFunction	= HenyeyGreensteinPhase(SLAB_SSSPHASEANISOTROPY(BSDFContext.BSDF), CosTheta);
				const float PhaseFunction	= HGPhaseFunction / IsotropicPhase();

				const float3 SlabDirectionalAlbedo = PhaseFunction * IsotropicMediumSlabPunctualDirectionalAlbedo(PM, BSDFContext.Context.NoV, BSDFContext.Context.NoL);

				const float RefractedTransmittanceNoV = SubstrateComputeTransmittanceNoV(F0, BSDFContext.V, BSDFContext.N, BSDFContext.Context.NoV);

				// Shading point <-> View throughput
				const float3 SlabTransmittanceV = IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, RefractedTransmittanceNoV);
				const float SpecularTransmissionV = DirectionalAlbedo_SpecularTransmission;

				// Shading point <-> Light throughput
				// Compute the transmittance at normal incidence (instead of BSDFContext.Context.NoL), and will compute the final value during evaluation 
				const float3 SlabTransmittanceN = IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, 1.f /*NoL with L==N*/);
				// The specular transmission is ignored towards the direction of the light. (TransmittanceAlongN can only store transmittance)

				// Now lerp between the optically thick and optically thin medium models.
				// The diffuse and throughput account for the GGX interface SpecularTransmission.
				Sample.DiffusePathValue		= lerp(Sample.DiffusePathValue,		SlabDirectionalAlbedo * SpecularTransmissionV,	DiffuseToVolumeBlend);
				Sample.DiffusePDF			= lerp(Sample.DiffusePDF,			HGPhaseFunction,								DiffuseToVolumeBlend);
				Sample.ThroughputV			= lerp(Sample.ThroughputV,			SlabTransmittanceV    * SpecularTransmissionV,	DiffuseToVolumeBlend);
				Sample.TransmittanceAlongN	= lerp(Sample.TransmittanceAlongN,	SlabTransmittanceN,								DiffuseToVolumeBlend);

				// Use Context.NoL which is not saturate, and allows to handle backface lighting, necessary for medium support
				#if SUBSTRATE_TRANSLUCENT_ENABLED
				const float MediumNoL = abs(BSDFContext.Context.NoL);
				#else
				const float MediumNoL = saturate(BSDFContext.Context.NoL);
				#endif

				Sample.IntegratedDiffuseValue = lerp(
					Sample.IntegratedDiffuseValue, 
					(ShadowTerms.SurfaceShadow * MediumNoL * AreaLight.Falloff) * AreaLight.FalloffColor * SlabDirectionalAlbedo * DirectionalAlbedo_SpecularTransmission,
					DiffuseToVolumeBlend);
				Sample.DiffuseColor		= SlabDirectionalAlbedo;
			}
			#endif

			
			////
			//// Evaluate cloth fuzz layered on top of the slab.
			////

			#if SUBSTRATE_FASTPATH==0
			BRANCH
			if (BSDF_GETHASFUZZ(BSDFContext.BSDF))
			{
				const float  FuzzAmount = SLAB_FUZZ_AMOUNT(BSDFContext.BSDF);
				const float3 FuzzF0  = SLAB_FUZZ_COLOR(BSDFContext.BSDF);
				const float  FuzzRoughness = MakeRoughnessSafe(SLAB_FUZZ_ROUGHNESS(BSDFContext.BSDF), SUBSTRATE_MIN_FUZZ_ROUGHNESS);

				// F0 is ignored, as energy preservation is handle solely based on the white directional albedo and FuzzAmount
				FBxDFEnergyTermsA ClothEnergyTerms = (FBxDFEnergyTermsA)1.f;
				float3 ClothSpecularPathValueNoL = 0;

				#if 1
				// Always use Sheen LTC as its cost is affordable and has better quality
				// Disney LTC
				{
					float DirectionalAlbedo = 1;
					BRANCH
					if (bIsRectLight)
					{
						ClothSpecularPathValueNoL = FuzzF0 * RectSheenApproxLTC(FuzzRoughness, BSDFContext.N, BSDFContext.V, AreaLight.Rect, AreaLight.Texture, DirectionalAlbedo);
					}
					else
					{
						ClothSpecularPathValueNoL = FuzzF0 * SheenLTC_Eval(BSDFContext.V, AreaLight.DiffuseL, BSDFContext.N, BSDFContext.Context.NoV, FuzzRoughness, View.SheenLTCTexture, View.SheenLTCSampler, DirectionalAlbedo);
						ClothSpecularPathValueNoL *= DirectionalAlbedo;
					}
					ClothEnergyTerms.E = DirectionalAlbedo;
					ClothEnergyTerms.W = 1.f;
				}
				#else
				// UE_DEPRECATED: Keep Charlie Sheen for now, but code will be removed in future release
				// Charlie cloth modle for D and Ashikhmin for visibility (more efficient than Charlie).
				{
					// ComputeEnergyConservation(ClothEnergyTerms) is not taken into account because we do not want white fuzz to reflect all incoming energy
					// The fuzz should have some transmission to the lower lobes. In order to pass a furnace test, a combination of white diffuse+white fuzz is required
					ClothEnergyTerms = ComputeClothEnergyTermsA(FuzzRoughness, BSDFContext.Context.NoV);

					float ClothD   = D_Charlie(FuzzRoughness, BSDFContext.Context.NoH);
					float ClothVis = Vis_Ashikhmin(BSDFContext.Context.NoV, AreaLight.NoL);
					float3 ClothF  = F_Schlick(FuzzF0, BSDFContext.Context.VoH);
					ClothSpecularPathValueNoL = (ClothD * ClothVis) * ClothF * ComputeEnergyConservation(ClothEnergyTerms) * AreaLight.NoL;
				}
				#endif //SUBSTRATE_SHEEN_QUALITY

				// The specular and diffuse components below the fuzz are only attenuated linearly according to the amount of fuzz.
				const float ClothEnegyPreservation = 1.f - ClothEnergyTerms.E;
				const float3 Cloth_DirectionalAlbedo_SpecularTransmission = lerp(1.0, ClothEnegyPreservation, FuzzAmount);

				// Area light are not supported by the cloth BRDF
				float3 ClothIntegratedSpecularValue = (ShadowTerms.SurfaceShadow * AreaLight.Falloff * FuzzAmount) * AreaLight.FalloffColor * ClothSpecularPathValueNoL;
				
				// Apply specular transmittance to diffuse and specular lob from slab medium and interface sitting below the layer of fuzz.
				Sample.DiffusePathValue			*= Cloth_DirectionalAlbedo_SpecularTransmission;
				Sample.IntegratedDiffuseValue	*= Cloth_DirectionalAlbedo_SpecularTransmission;
				Sample.SpecularPathValue		*= Cloth_DirectionalAlbedo_SpecularTransmission;
				Sample.IntegratedSpecularValue	*= Cloth_DirectionalAlbedo_SpecularTransmission;
				
				Sample.SpecularPathValue		+= ClothSpecularPathValueNoL;
				Sample.IntegratedSpecularValue  += ClothIntegratedSpecularValue;

				// This is not good. We should really have a separate PDF for cloth it self associated with a probability.
				// We should also output probability to sample cloth, diffuse, secular, etc.
				// We will revisit when the path tracer is getting up with Substrate.
				//Sample.SpecularPDF = lerp(Sample.SpecularPDF, BSDFContext.SatNoV / PI, FuzzAmount); // Per "Production Friendly Microfacet Sheen BRDF", hemispherical sampling give good result as the roughness is usually high.
				// SUBSTRATE_TODO we should have a separated cloth PDF and Weight

				if (bHaziness)
				{
					Sample.SpecularHazePathValue*= Cloth_DirectionalAlbedo_SpecularTransmission;
					Sample.SpecularHazePathValue+= ClothSpecularPathValueNoL;

					// SUBSTRATE_TODO we should have a separated cloth PDF and Weight
					//Sample.SpecularHazePDF		= lerp(Sample.SpecularHazePDF,			BSDFContext.SatNoV / PI,	Fuzz);;
				}

				Sample.ThroughputV *= Cloth_DirectionalAlbedo_SpecularTransmission;
				// The specular transmission is ignored towards the direction of the light. (TransmittanceAlongN can only store transmittance)
			}
			#endif

			break;
		}

		#if SUBSTRATE_FASTPATH==0
		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
			FGBufferData GBuffer	= (FGBufferData)0;
			GBuffer.BaseColor		= HAIR_BASECOLOR(BSDFContext.BSDF);
			GBuffer.Specular		= HAIR_SPECULAR(BSDFContext.BSDF);
			GBuffer.Roughness		= HAIR_ROUGHNESS(BSDFContext.BSDF);
			GBuffer.Metallic		= HAIR_SCATTER(BSDFContext.BSDF);
			GBuffer.CustomData.z	= HAIR_BACKLIT(BSDFContext.BSDF);
			GBuffer.ShadingModelID	= SHADINGMODELID_HAIR;
			GBuffer.WorldNormal		= BSDFContext.N;

			FHairTransmittanceData HairTransmittance = InitHairTransmittanceData();
			if (Settings.bUseShadowTermHairTransmittance)
			{
				HairTransmittance = ShadowTerms.HairTransmittance;
				HairTransmittance.OpaqueVisibility = ShadowTerms.SurfaceShadow;
			}
			else if (HAIR_COMPLEXTRANSMITTANCE(BSDFContext.BSDF))
			{
				HairTransmittance = EvaluateDualScattering(GBuffer.BaseColor, BSDFContext.N, GBuffer.Roughness, BSDFContext.V, BSDFContext.L);
				HairTransmittance.OpaqueVisibility = ShadowTerms.SurfaceShadow;
			}

			float BacklitEnabled = 1.0f;
			float Area = 0.0f;
			uint2 Random = uint2(0, 0);
			Sample.SpecularPathValue	= HairShading(GBuffer, BSDFContext.L, BSDFContext.V, BSDFContext.N, ShadowTerms.TransmissionShadow, HairTransmittance, BacklitEnabled, Area, Random);
			Sample.SpecularPDF			= 1.0f / (4.0f * PI);		// SUBSTRATE_TODO this currently match the uniform sphere sampling from SubstrateImportanceSampleBSDF
			Sample.ThroughputV			= OpaqueBSDFThroughput; 
			Sample.TransmittanceAlongN	= OpaqueBSDFThroughput;
			Sample.IntegratedSpecularValue = (ShadowTerms.TransmissionShadow * AreaLight.Falloff) * AreaLight.FalloffColor * Sample.SpecularPathValue;
		}
		break;
		
		case SUBSTRATE_BSDF_TYPE_EYE:
		{
			float3 DiffuseColor	= EYE_DIFFUSEALBEDO(BSDFContext.BSDF);
			float3 F0			= EYE_F0(BSDFContext.BSDF);
			float3 F90			= EYE_F90(BSDFContext.BSDF);
			const float SafeRoughness	= MakeRoughnessSafe(EYE_ROUGHNESS(BSDFContext.BSDF), SUBSTRATE_MIN_GGX_ROUGHNESS);

			// Specular occlusion is only used here once to affect the F90 source parameters. F0 will decrease naturally to 0.
			F90 *= F0RGBToMicroOcclusion(F0);

			if (Settings.bForceFullyRough)
			{
				// When rendering reflection captures, the BSDF roughness has already been forced to 1 using View.RoughnessOverrideParameter (see SubstrateSanitizeBSDF).
				EnvBRDFApproxFullyRough(DiffuseColor, F0, F90);
			}

			float Alpha2 = Pow4(SafeRoughness);

			Init(BSDFContext.Context, BSDFContext.N, BSDFContext.V, AreaLight.SpecularL);
			SphereMaxNoH(BSDFContext.Context, AreaLight.SphereSinAlpha, false);

			BSDFContext.Context.NoV = saturate(max(abs(BSDFContext.Context.NoV), SUBSTRATE_EPSILON));

			////
			//// Evaluate the diffuse component.
			////

			const float IrisNoL = saturate(dot(EYE_IRISNORMAL(BSDFContext.BSDF), BSDFContext.L));

			// Blend in the negative intersection normal to create some concavity
			// Not great as it ties the concavity to the convexity of the cornea surface
			// No good justification for that. On the other hand, if we're just looking to
			// introduce some concavity, this does the job.
			const float3 CausticNormal = normalize(lerp(EYE_IRISPLANENORMAL(BSDFContext.BSDF), -BSDFContext.N, EYE_IRISMASK(BSDFContext.BSDF) * EYE_IRISDISTANCE(BSDFContext.BSDF)));

			// Add-hoc legacy shading mode for eye. 
			const float Power = lerp(12, 1, IrisNoL);
			const float Caustic = 0.8 + 0.2 * (Power + 1) * pow(saturate(dot(CausticNormal, BSDFContext.L)), Power);

			const float Iris   = IrisNoL * Caustic;
			const float Sclera = AreaLight.NoL;

			Sample.DiffusePathValue = Diffuse_Lambert(DiffuseColor) * lerp(Sclera, Iris, EYE_IRISMASK(BSDFContext.BSDF));
			Sample.IntegratedDiffuseValue += (ShadowTerms.TransmissionShadow * /* We don't AreaLight.NoL as it is already applied through the Sclera/Iris value above */ AreaLight.Falloff) * AreaLight.FalloffColor * Sample.DiffusePathValue;

			Sample.DiffuseColor		= DiffuseColor;
			Sample.DiffusePDF		= BSDFContext.SatNoL / PI;
			Sample.bPostProcessSubsurface = (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION || BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION_PROFILE) && SUBSTRATE_SSS_MATERIAL_OVERRIDE;

			//
			// Apply energy conservation on the diffuse component
			// If the specular layer is anisotropic, the energy term is computed onto the 'main' roughness [Kulla 2019]
			//
			const float3 FresnelTerm = Substrate_F_GGX(F0, F90, BSDFContext.Context.VoH);
			FBxDFEnergyType MSScale = 1;
			float DirectionalAlbedo_SpecularTransmission = 1.0f;
			#if USE_ENERGY_CONSERVATION
			{
				FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(SafeRoughness, BSDFContext.Context.NoV, F0, F90);
				DirectionalAlbedo_SpecularTransmission = ComputeEnergyPreservation(EnergyTerms);
				MSScale = ComputeEnergyConservation(EnergyTerms);
			}
			#else
			{
				// Preserve old behavior when energy conservation is disabled, which is the case with Format=0
				DirectionalAlbedo_SpecularTransmission = 1.f - FresnelTerm.x;
			}
			#endif
			
			Sample.DiffusePathValue *= DirectionalAlbedo_SpecularTransmission;
			Sample.IntegratedDiffuseValue *= DirectionalAlbedo_SpecularTransmission;

			////
			//// Evaluate the specular component.
			//// This takes into account multiple scattering, micro occlusion.
			//// Note: anisotropy completely disables area integrations. Lights fall back to punctual.
			////

			float3 RectLightSpec = 0;
			float3 RectLightSpecHaze = 0;
			float D = 0;
			float Vis = 0;
			float PDF = 0;
			const bool bIsRectLight = IntegrationType == INTEGRATION_AREA_LIGHT_RECT;

			if (bIsRectLight)
			{
				// In this case, we set D to 1 and Vis will contain the area light / GGX integration
				RectLightSpec = RectGGXApproxLTC(SafeRoughness, F0, F90, BSDFContext.N, BSDFContext.V, AreaLight.Rect, AreaLight.Texture);

				const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Alpha2);
				PDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
			}
			else
			{
				// Special override for roughness for supporting area light integrator with Sphere/Tube/Disk light, which modifies/increase roughness.

				// Generalized microfacet specular
				float Alpha2Spec = Alpha2;
				const float Energy = EnergyNormalization(Alpha2Spec, BSDFContext.Context.VoH, AreaLight);
				D = Substrate_D_GGX(SafeRoughness, Alpha2Spec, BSDFContext.Context.NoH) * Energy;
				Vis = Substrate_Vis_GGX(SafeRoughness, Alpha2Spec, BSDFContext.Context.NoV, AreaLight.NoL);
				const float H_PDF = VisibleGGXPDF(BSDFContext.TangentV, BSDFContext.TangentH, Alpha2Spec);
				PDF = RayPDFToReflectionRayPDF(BSDFContext.Context.VoH, H_PDF);
			}


			Sample.SpecularPathProbability	= 1.0f;
			Sample.SpecularPDF				= PDF * Sample.SpecularPathProbability;

			if (bIsRectLight)
			{
				Sample.SpecularPathValue       = RectLightSpec * MSScale;
				Sample.IntegratedSpecularValue = (ShadowTerms.SurfaceShadow * /* AreaLight.NoL and falloff is part of the LTC integration already */ Sample.SpecularPathProbability) * Sample.SpecularPathValue;
			}
			else
			{
				Sample.SpecularPathValue        = D * Vis * MSScale * FresnelTerm;
				Sample.IntegratedSpecularValue += (ShadowTerms.SurfaceShadow * AreaLight.NoL * AreaLight.Falloff) * AreaLight.FalloffColor * Sample.SpecularPathValue * Sample.SpecularPathProbability;
			}

			////
			//// Evaluate emissive and set the sample throughput (transmittance to next layer) corresponding to an opaque slab of matter.
			////
			
			// we do not need to add emissive for the BRDF TotalSpec or TotalDiff values as this is handled separately
			Sample.EmissivePathValue = BSDF_GETEMISSIVE(BSDFContext.BSDF);

			Sample.ThroughputV			= OpaqueBSDFThroughput;
			Sample.TransmittanceAlongN	= OpaqueBSDFThroughput;

			break;
		}
		break;

		case SUBSTRATE_BSDF_TYPE_TOON:
		{
			const float3 DiffuseColor		= ComputeDiffuseAlbedo(TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float3 SpecularColor		= ComputeF0(TOON_SPECULAR(BSDFContext.BSDF), TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float  Roughness			= TOON_ROUGHNESS(BSDFContext.BSDF);

			const float ToonProfileId		= TOON_PROFILEID(BSDFContext.BSDF);
			const float2 ToonPatternUVs		= TOON_PATTERNUVS(BSDFContext.BSDF);

			const float Anisotropy			= TOON_ANISOTROPY(BSDFContext.BSDF);
			const float3 Emissive			= BSDF_GETEMISSIVE(BSDFContext.BSDF);

			const bool bThisPassAppliesDiffuseRamp	= SUBSTRATE_LIGHTPASS_APPLIES_TOON_DIFFUSE_RAMP;
			const bool bHasAnisotropy				= BSDF_GETHASANISOTROPY(BSDFContext.BSDF) && Anisotropy != 0;
			const bool bIsRectLight					= IntegrationType == INTEGRATION_AREA_LIGHT_RECT;

			FSubstrateEvaluateToonResult ToonResult = SubstrateToonEvaluateCommon(
				DiffuseColor,
				SpecularColor,
				Roughness,
				ToonProfileId,
				ToonPatternUVs,
				Anisotropy,
				Emissive,
				BSDFContext.Context,
				bHasAnisotropy,
				bIsRectLight,
				bThisPassAppliesDiffuseRamp,
				BSDFContext.N, BSDFContext.V, AreaLight, ShadowTerms.SurfaceShadow, ShadowTerms.TransmittanceOrOpticalThickness);

			// Forward to FSubstrateEvaluateResult
			Sample.IntegratedDiffuseValue	= ToonResult.IntegratedDiffuseValue;
			Sample.IntegratedSpecularValue	= ToonResult.IntegratedSpecularValue;

			Sample.DiffuseColor				= ToonResult.DiffuseColor;

			Sample.EmissivePathValue		= ToonResult.EmissivePathValue;

			Sample.DiffusePathValue			= ToonResult.DiffusePathValue;
			Sample.SpecularPathValue		= ToonResult.SpecularPathValue;

			Sample.DiffusePDF				= ToonResult.DiffusePDF;

			Sample.ThroughputV				= ToonResult.ThroughputV;
			Sample.TransmittanceAlongN		= ToonResult.TransmittanceAlongN;
			Sample.bPostProcessSubsurface	= ToonResult.bPostProcessSubsurface;

			Sample.ToonData.Weight			= ToonResult.ToonDataWeight;
			Sample.ToonData.ShadowNoL		= ToonResult.ToonDataShadowNoL;
		}
		break;

		//case SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD:
		//case SUBSTRATE_BSDF_TYPE_UNLIT:
		//case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		//Nothing to do in this case because these BSDF are evaluated in other specialised passes.
		//break;
		#endif
	}

	return Sample;
}

// Punctual light evaluation
FSubstrateEvaluateResult SubstrateEvaluateBSDF(FSubstrateBSDFContext BSDFContext, FSubstrateIntegrationSettings Settings)
{
	FShadowTerms IdentityShadow = { 1, 1, 1, InitHairTransmittanceData() };
	FAreaLight DummyAreaLight = InitAreaLight();
	DummyAreaLight.DiffuseL = BSDFContext.L;
	DummyAreaLight.SpecularL = BSDFContext.L;
	DummyAreaLight.NoL = BSDFContext.Context.NoL;
	DummyAreaLight.Falloff = 1;
	return SubstrateEvaluateBSDFCommon(BSDFContext, IdentityShadow, DummyAreaLight, Settings, INTEGRATION_PUNCTUAL_LIGHT);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// BSDF Importance sampling
 
// SUBSTRATE_TODO: merge this with Path-Tracing (watch out -- implementations have diverged)

// Given two lobes that will roughly contribute colors A and B to the total (estimated for example using directional albedo)
// return the probability of choosing lobe A
float SubstrateLobeSelectionProb(float3 A, float3 B)
{
	const float SumA = A.x + A.y + A.z;
	const float SumB = B.x + B.y + B.z;
	return SumA / (SumA + SumB + 1e-6);
}

// Select a term based based on a random value and a PDF, and returns the selected term
// The random value is rescale/updated to reusable after evaluation
#define SubstrateSelectLobe_Type(TDataType) TDataType SubstrateSelectLobe(inout float InE, float InPDF, TDataType InA, TDataType InB)\
{\
	if (InE < InPDF){ InE /= InPDF;							return InA; }\
	else			{ InE -= InPDF; InE /= (1.0 - InPDF);	return InB; }\
}
SubstrateSelectLobe_Type(uint)
SubstrateSelectLobe_Type(float)


/**
 * Importance sample a Substrate BSDF
 * BSDF: the Substrate BSDF to importance sample
 * E: two random numbers
 * CameraVector: vector from the camera to the considered direction (not V)
 */
#define FSubstrateImportanceSampleResult FBxDFSample
FSubstrateImportanceSampleResult SubstrateImportanceSampleBSDF(FSubstrateBSDFContext BSDFContext, float2 E, uint InTermMask, FSubstrateIntegrationSettings Settings)
{
	FSubstrateImportanceSampleResult Out = (FSubstrateImportanceSampleResult)0;
	switch (BSDF_GETTYPE(BSDFContext.BSDF))
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			const float3 DiffuseColor	= SLAB_DIFFUSEALBEDO(BSDFContext.BSDF);
			const float3 F0 			= SLAB_F0(BSDFContext.BSDF);
			const float3 F90 			= SLAB_F90(BSDFContext.BSDF);

			const bool bHasAnisotropy	= BSDF_GETHASANISOTROPY(BSDFContext.BSDF) && SUPPORTS_ANISOTROPIC_MATERIALS;
			const bool bHaziness		= BSDF_GETHASHAZINESS(BSDFContext.BSDF);
			const float SafeRoughness   = MakeRoughnessSafe(SLAB_ROUGHNESS(BSDFContext.BSDF));

			// Pick diffuse/specular lobe
			float TermPDF = Out.Term == SHADING_TERM_DIFFUSE ? 1 : 0;
			Out.Term = InTermMask;
			if (Out.Term == (SHADING_TERM_DIFFUSE | SHADING_TERM_SPECULAR))
			{
				// Use (main) specular lobe as a heuristic to pick select between specular/diffuse lobe
				const FBxDFEnergyTerms SpecEnergyTerms = ComputeGGXSpecEnergyTerms(SafeRoughness, BSDFContext.Context.NoV, F0, F90);

				// Probability of picking diffuse lobe vs. specular lobe
				TermPDF  = SubstrateLobeSelectionProb(DiffuseColor * (1 - SpecEnergyTerms.E), SpecEnergyTerms.E);
				Out.Term = SubstrateSelectLobe(E.x, TermPDF, SHADING_TERM_DIFFUSE, SHADING_TERM_SPECULAR);
			}

			// Pick specular lobe
			float SpecularLobePDF = 1.f;
			float SpecularRoughness = SafeRoughness;
			if (Out.Term == SHADING_TERM_SPECULAR && bHaziness)
			{
				FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
				const float HazeWeight = Haziness.Weight;
				const float HazeSafeRoughness = MakeRoughnessSafe(Haziness.Roughness);

				SpecularLobePDF   = 1.f - HazeWeight;
				SpecularRoughness = SubstrateSelectLobe(E.x, SpecularLobePDF, SafeRoughness, HazeSafeRoughness);
			}

			// Sample Diffuse lobe
			if (Out.Term == SHADING_TERM_DIFFUSE)
			{
				float4 ImportanceSample = CosineSampleHemisphere(E);
				Out.L   = mul(ImportanceSample.xyz, BSDFContext.TangentBasis);
				Out.PDF = ImportanceSample.w;
			}

			// Sample Specular lobe
			if (Out.Term == SHADING_TERM_SPECULAR)
			{
				float2 Alpha = Pow2(SpecularRoughness);
				if (bHasAnisotropy)
				{
					GetAnisotropicRoughness(Alpha.x, SLAB_ANISOTROPY(BSDFContext.BSDF), Alpha.x, Alpha.y);
				}

				float4 GGXSample	= ImportanceSampleVisibleGGX(E, Alpha, BSDFContext.TangentV);
				const float HPDF	= GGXSample.w;
				const float3 H		= mul(GGXSample.xyz, BSDFContext.TangentBasis);
				const float VoH		= saturate(dot(BSDFContext.V, H));
	
				Out.L				= 2 * dot(BSDFContext.V, H) * H - BSDFContext.V;
				Out.PDF				= RayPDFToReflectionRayPDF(VoH, HPDF);
			}

			// Evaluate lobes
			BSDFContext.SubstrateUpdateBSDFContext(Out.L);
			const FSubstrateEvaluateResult Evaluate = SubstrateEvaluateBSDF(BSDFContext, Settings);

			// Add lobes mixtures
			Out.PDF = 0;
			Out.Weight = 0;
			if (InTermMask & SHADING_TERM_DIFFUSE)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.DiffusePathValue / Evaluate.DiffusePDF, Evaluate.DiffusePDF, TermPDF);
			}

			if (InTermMask & SHADING_TERM_SPECULAR)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.SpecularPathValue / Evaluate.SpecularPDF, Evaluate.SpecularPDF, (1-TermPDF) * SpecularLobePDF);
				if (bHaziness)
				{
					AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.SpecularHazePathValue / Evaluate.SpecularHazePDF, Evaluate.SpecularPDF, (1 - TermPDF) * (1-SpecularLobePDF));
				}
			}

			// SUBSTRATE_TODO take into account Two sided material
			// SUBSTRATE_TODO take into account Transmission for path-tracing
			break;
		}

		case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		{
			const float SafeRoughness = MakeRoughnessSafe(SLW_ROUGHNESS(BSDFContext.BSDF));
			const float4 TangentH = ImportanceSampleVisibleGGX(E, Pow2(SafeRoughness), BSDFContext.TangentV);

			const float HPDF = TangentH.w;
			const float3 H = mul(TangentH.xyz, BSDFContext.TangentBasis);
			const float VoH = saturate(dot(BSDFContext.V, H));

			Out.L = 2 * dot(BSDFContext.V, H) * H - BSDFContext.V;
			Out.PDF = RayPDFToReflectionRayPDF(VoH, HPDF);
			Out.Term = SHADING_TERM_SPECULAR;
			Out.Weight = 1.f;
			break;
		}

		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
			// SUBSTRATE_TODO do something better when we get there with Lumen, and evaluate the different researched solution (e.g. Importance Sampling for Physically-Based Hair Fiber Models)
			float4 ImportanceSample = CosineSampleHemisphere(E);
			Out.L    = mul(ImportanceSample.xyz, BSDFContext.TangentBasis);
			Out.PDF  = ImportanceSample.w;
			Out.Term = SHADING_TERM_SPECULAR;
			Out.Weight = 1.f;
			break;
		}

		case SUBSTRATE_BSDF_TYPE_EYE:
		{
			// TODO manage iris normal for the diffuse term
			const float3 DiffuseColor	= EYE_DIFFUSEALBEDO(BSDFContext.BSDF);
			const float3 F0 			= EYE_F0(BSDFContext.BSDF);
			const float3 F90			= EYE_F90(BSDFContext.BSDF);
			const float SafeRoughness   = MakeRoughnessSafe(EYE_ROUGHNESS(BSDFContext.BSDF));

			// Pick diffuse/specular lobe
			float TermPDF = Out.Term == SHADING_TERM_DIFFUSE ? 1 : 0;
			Out.Term = InTermMask;
			if (Out.Term == (SHADING_TERM_DIFFUSE | SHADING_TERM_SPECULAR))
			{
				// Use (main) specular lobe as a heuristic to pick select between specular/diffuse lobe
				const FBxDFEnergyTerms SpecEnergyTerms = ComputeGGXSpecEnergyTerms(SafeRoughness, BSDFContext.Context.NoV, F0, F90);

				// Probability of picking diffuse lobe vs. specular lobe
				TermPDF  = SubstrateLobeSelectionProb(DiffuseColor * (1 - SpecEnergyTerms.E), SpecEnergyTerms.E);
				Out.Term = SubstrateSelectLobe(E.x, TermPDF, SHADING_TERM_DIFFUSE, SHADING_TERM_SPECULAR);
			}

			// Sample Diffuse lobe
			if (Out.Term == SHADING_TERM_DIFFUSE)
			{
				float4 ImportanceSample = CosineSampleHemisphere(E);
				Out.L   = mul(ImportanceSample.xyz, BSDFContext.TangentBasis);
				Out.PDF = ImportanceSample.w;
			}

			// Sample Specular lobe
			if (Out.Term == SHADING_TERM_SPECULAR)
			{
				float4 TangentH = ImportanceSampleVisibleGGX(E, Pow2(SafeRoughness), BSDFContext.TangentV);

				const float HPDF = TangentH.w;
				const float3 H = mul(TangentH.xyz, BSDFContext.TangentBasis);
				const float VoH = saturate(dot(BSDFContext.V, H));

				Out.L   = 2 * dot(BSDFContext.V, H) * H - BSDFContext.V;
				Out.PDF = RayPDFToReflectionRayPDF(VoH, HPDF);
			}

			// Evaluate lobes
			BSDFContext.SubstrateUpdateBSDFContext(Out.L);
			const FSubstrateEvaluateResult Evaluate = SubstrateEvaluateBSDF(BSDFContext, Settings);

			// Add lobes mixtures
			Out.PDF = 0;
			Out.Weight = 0;
			if (InTermMask & SHADING_TERM_DIFFUSE)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.DiffusePathValue / Evaluate.DiffusePDF, Evaluate.DiffusePDF, TermPDF);
			}

			if (InTermMask & SHADING_TERM_SPECULAR)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.SpecularPathValue / Evaluate.SpecularPDF, Evaluate.SpecularPDF, (1-TermPDF));
			}
			break;
		}

		case SUBSTRATE_BSDF_TYPE_TOON:
		{
			const float3 DiffuseColor						= ComputeDiffuseAlbedo(TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float3 F0									= ComputeF0(TOON_SPECULAR(BSDFContext.BSDF), TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float  F90								= 1.0f;
			const float SafeRoughness						= MakeRoughnessSafe(TOON_ROUGHNESS(BSDFContext.BSDF));

			// Pick diffuse/specular lobe
			float TermPDF = Out.Term == SHADING_TERM_DIFFUSE ? 1 : 0;
			Out.Term = InTermMask;
			if (Out.Term == (SHADING_TERM_DIFFUSE | SHADING_TERM_SPECULAR))
			{
				// Use (main) specular lobe as a heuristic to pick select between specular/diffuse lobe
				const FBxDFEnergyTerms SpecEnergyTerms = ComputeGGXSpecEnergyTerms(SafeRoughness, BSDFContext.Context.NoV, F0, F90);

				// Probability of picking diffuse lobe vs. specular lobe
				TermPDF = SubstrateLobeSelectionProb(DiffuseColor * (1 - SpecEnergyTerms.E), SpecEnergyTerms.E);
				Out.Term = SubstrateSelectLobe(E.x, TermPDF, SHADING_TERM_DIFFUSE, SHADING_TERM_SPECULAR);
			}

			// Pick specular lobe
			float SpecularLobePDF = 1.f;
			float SpecularRoughness = SafeRoughness;

			// Sample Diffuse lobe
			if (Out.Term == SHADING_TERM_DIFFUSE)
			{
				float4 ImportanceSample = CosineSampleHemisphere(E);
				Out.L = mul(ImportanceSample.xyz, BSDFContext.TangentBasis);
				Out.PDF = ImportanceSample.w;
			}

			// Sample Specular lobe
			if (Out.Term == SHADING_TERM_SPECULAR)
			{
				float2 Alpha = Pow2(SpecularRoughness);

				float4 GGXSample = ImportanceSampleVisibleGGX(E, Alpha, BSDFContext.TangentV);
				const float HPDF = GGXSample.w;
				const float3 H = mul(GGXSample.xyz, BSDFContext.TangentBasis);
				const float VoH = saturate(dot(BSDFContext.V, H));

				Out.L = 2 * dot(BSDFContext.V, H) * H - BSDFContext.V;
				Out.PDF = RayPDFToReflectionRayPDF(VoH, HPDF);
			}

			// Evaluate lobes
			BSDFContext.SubstrateUpdateBSDFContext(Out.L);
			const FSubstrateEvaluateResult Evaluate = SubstrateEvaluateBSDF(BSDFContext, Settings);

			// Add lobes mixtures
			Out.PDF = 0;
			Out.Weight = 0;
			if (InTermMask & SHADING_TERM_DIFFUSE)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.DiffusePathValue / Evaluate.DiffusePDF, Evaluate.DiffusePDF, TermPDF);
			}

			if (InTermMask & SHADING_TERM_SPECULAR)
			{
				AddLobeWithMIS(Out.Weight, Out.PDF, Evaluate.SpecularPathValue / Evaluate.SpecularPDF, Evaluate.SpecularPDF, (1 - TermPDF) * SpecularLobePDF);
			}
			break;
		}
		break;

		//case SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD:
		//case SUBSTRATE_BSDF_TYPE_UNLIT:
		//Nothing to do in this case because these BSDF are evaluated in other specialised passes.
		//break;
	}
	return Out;
}

// Energy conservation used for env. evaluation. 
// Format=0 disables energy conservation for performance reason. However In order to keep env. evalution correct, we force simple energy lookup).
FBxDFEnergyTermsRGB ComputeGGXSpecEnergyTermsRGB_Env(float SafeRoughness, float NoV, float3 F0, float3 F90)
{
	#if USE_ENERGY_CONSERVATION > 0
	return ComputeGGXSpecEnergyTermsRGB(SafeRoughness, NoV, F0, F90);
	#else
	return ComputeFresnelEnergyTermsRGB(GGXEnergyLookupEnv(SafeRoughness, NoV), F0, F90);
	#endif
}

struct FSubstrateEnvLightResult
{
	float3 DiffuseNormal;
	float3 DiffuseWeight;
	float3 DiffuseBackFaceWeight;

	float3 DiffuseColor;
	float3 SpecularColor;

	float3 SpecularDirection;
	float3 SpecularWeight;
	float  SpecularSafeRoughness;

	float3 SpecularHazeDirection;	// We must use a different direction to not have a rough specular affect a sharp specular due to SubstrateGetOffSpecularPeakReflectionDir.
	float3 SpecularHazeWeight;
	float  SpecularHazeSafeRoughness;
	float  SSRReduction;

	// This can vary depending if the simple clear coat is used.
	float3 SSRSpecularWeight;

#if SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION
	float ToonProfileID;
#endif

	bool   bPostProcessSubsurface; // True if we need to separate the subsurface light contribution for the screen space diffusion process.
};

FSubstrateEnvLightResult SubstrateEvaluateForEnvLight(FSubstrateBSDFContext BSDFContext, bool bEnableSpecular, FSubstrateIntegrationSettings Settings)
{
	FSubstrateEnvLightResult SubstrateEnvLightResult = (FSubstrateEnvLightResult)0;
#if SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION
	SubstrateEnvLightResult.ToonProfileID = -1.0f;
#endif
	const uint BSDFType = BSDF_GETTYPE(BSDFContext.BSDF);
	switch (BSDFType)
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			float3 DiffuseColor		= SLAB_DIFFUSEALBEDO(BSDFContext.BSDF);
			float3 F0 				= SLAB_F0(BSDFContext.BSDF);
			float3 F90 				= SLAB_F90(BSDFContext.BSDF);

			// Specular occlusion is only used here once to affect the F90 source parameters. F0 will decrease naturally to 0.
			F90 *= F0RGBToMicroOcclusion(F0);

			SubstrateEnvLightResult.DiffuseNormal = BSDFContext.N;
			SubstrateEnvLightResult.bPostProcessSubsurface = false;

			float SafeRoughness = MakeRoughnessSafe(SLAB_ROUGHNESS(BSDFContext.BSDF));

			// When rendering reflection captures, the BSDF roughness has already been forced to 1 using View.RoughnessOverrideParameter (see SubstrateSanitizeBSDF).
			if (Settings.bForceFullyRoughAccurate)
			{
				SafeRoughness = 1.f;
			}
			else if (Settings.bForceFullyRough)
			{
				// The version with EnvBRDFApproxFullyRough is faster, but leads to less accurate result in particular 
				// when dealing with specular response over diffuse response.
				EnvBRDFApproxFullyRough(DiffuseColor, F0, F90);
			}

			float3 EvalEnvBRDF = 0;
			float DirectionalAlbedo_SpecularTransmission = 0;
			{
				FBxDFEnergyTermsRGB SpecularEnergyTerms = ComputeGGXSpecEnergyTermsRGB_Env(SafeRoughness, BSDFContext.Context.NoV, F0, F90);
				FBxDFEnergyTermsRGB DiffuseEnergyTerms  = ComputeDiffuseEnergyTermsRGB(SafeRoughness, BSDFContext.Context.NoV);
				EvalEnvBRDF								= SpecularEnergyTerms.E;
				DirectionalAlbedo_SpecularTransmission	= ComputeEnergyPreservation(SpecularEnergyTerms);
				SubstrateEnvLightResult.DiffuseWeight	= DiffuseColor * DiffuseEnergyTerms.E;
				SubstrateEnvLightResult.DiffuseColor	= DiffuseColor;
			}

			#if SUBSTRATE_FASTPATH==0
			const bool bIsSubsurfaceDiffusionProfile = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION_PROFILE;
			const bool bIsSubsurfaceDiffusionMFP = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION;
			SubstrateEnvLightResult.bPostProcessSubsurface = (bIsSubsurfaceDiffusionProfile || bIsSubsurfaceDiffusionMFP) && SUBSTRATE_SSS_MATERIAL_OVERRIDE;

			#if SUBSTRATE_SUPPORTS_SIMPLEVOLUME
			BRANCH
			if (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_SIMPLEVOLUME)
			{
				FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(DiffuseColor, SLAB_SSSMFP(BSDFContext.BSDF));
				const float DiffuseToVolumeBlend = SubstrateSlabDiffuseToVolumeBlend(PM);

				float3 SlabDirectionalAlbedo = IsotropicMediumSlabEnvDirectionalAlbedo(PM, BSDFContext.Context.NoV, BSDFContext.PixelCoord);

			#if 1
				// Because the default EnvBRDF does not account for multiple scattering, we rescale the participating media response according to EnvBRDF.
				// This is to ensure a monotone light response behavior when transitioning from the Diffuse to the Volumetric model (without that, multiple scattering can result in higher luminance while trnasitionning from diffuse to volume light model)
				// SUBSTRATE_TODO: We should NOT have to do that. This should be investigated futher...
				const float MaxDiffuseWeight = max3(SubstrateEnvLightResult.DiffuseWeight.x, SubstrateEnvLightResult.DiffuseWeight.y, SubstrateEnvLightResult.DiffuseWeight.z);
				const float MaxSlabDirectionalAlbedo = max3(SlabDirectionalAlbedo.x, SlabDirectionalAlbedo.y, SlabDirectionalAlbedo.z);
				if (MaxDiffuseWeight < MaxSlabDirectionalAlbedo)
				{
					// We only want to do that when the slab light response is higher than the diffse response, so that when the mfp is large (and scattering low), the light response still work correctly according to thickness and MFP.
					SlabDirectionalAlbedo *= MaxDiffuseWeight / MaxSlabDirectionalAlbedo;
				}
			#endif

				SubstrateEnvLightResult.DiffuseWeight = lerp(SubstrateEnvLightResult.DiffuseWeight, SlabDirectionalAlbedo, DiffuseToVolumeBlend);
				// We keep the same normal
			}
			#endif
			#endif
			

			BRANCH
			if (bEnableSpecular)
			{
				// Hazy specular is top specular, non hazy is bottom with potentially bottom normal.
				// Save N/R/NoV before they are modified by anisotropy
				#if SUBSTRATE_FASTPATH==0
				const float3 ClearCoatTopContextN   = BSDFContext.N;
				const float3 ClearCoatTopContextR   = BSDFContext.R;
				const float  ClearCoatTopContextNoV = BSDFContext.Context.NoV;
				float ClearCoatBottomContextNoL = 0;
				float ClearCoatBottomContextNoV = 0;
				const bool bHasHaziness = BSDF_GETHASHAZINESS(BSDFContext.BSDF);
				if (bHasHaziness)
				{
					BxDFContext ClearCoatBottomContext = RefractClearCoatContext(BSDFContext.Context);
					ClearCoatBottomContextNoL = ClearCoatBottomContext.NoL;
					ClearCoatBottomContextNoV = ClearCoatBottomContext.NoV;

				#if SUBSTRATE_FASTPATH==0 && CLEAR_COAT_BOTTOM_NORMAL
					BRANCH
					if (BSDF_GETHASHAZINESS(BSDFContext.BSDF))
					{
						FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
						const bool  bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;
						BRANCH
						if (bHazeAsSimpleClearCoat && Haziness.HasBottomNormal())
						{
							float3 BottomNormal = SubstrateGetSimpleCoatBottomNormal(Haziness.BottomNormalOct, BSDFContext.N);

							BSDFContext.TangentBasis[2] = BottomNormal;
							BSDFContext = SubstrateCreateBSDFContext(BSDFContext.BSDF, BSDFContext.TangentBasis, BSDFContext.V, BSDFContext.R/*L*/);

							SubstrateEnvLightResult.DiffuseNormal = BottomNormal;
						}
					}
				#endif
				}
				#endif

				#if SUBSTRATE_COMPLEXPATH
				const bool bHasAnisotropy = BSDF_GETHASANISOTROPY(BSDFContext.BSDF) && SUPPORTS_ANISOTROPIC_MATERIALS;
				if (bHasAnisotropy)
				{
					// Modified the BSDF normal (and roughness)
					const float Anisotropy = SLAB_ANISOTROPY(BSDFContext.BSDF);			
					ModifyGGXAnisotropicNormalRoughness(BSDFContext.X, Anisotropy, SafeRoughness, BSDFContext.N, BSDFContext.V);

					// Update context (only needs: NoL/SatNoL/R) with the new N
					BSDFContext.Context.NoL = dot(BSDFContext.N, BSDFContext.L);
					BSDFContext.SatNoL = saturate(BSDFContext.Context.NoL);
					BSDFContext.R = 2 * dot(BSDFContext.V, BSDFContext.N) * BSDFContext.N - BSDFContext.V;
					BSDFContext.TangentBasis[2] = BSDFContext.N;
				}
				#endif

				SubstrateEnvLightResult.SpecularDirection = SubstrateGetOffSpecularPeakReflectionDir(BSDFContext.N, BSDFContext.R, SafeRoughness);
				SubstrateEnvLightResult.SpecularWeight = EvalEnvBRDF;
				SubstrateEnvLightResult.SpecularSafeRoughness = SafeRoughness;
				SubstrateEnvLightResult.SpecularColor = F0;

				// Glint shading
			#if SUBSTRATE_GLINTS_ALLOWED && SUBSTRATE_GLINTS_ENABLED
				{
					SubstrateEnvLightResult.SpecularWeight *= EvaluateGlintEnv(BSDFContext, SubstrateEnvLightResult.SpecularSafeRoughness, SubstrateEnvLightResult.SpecularDirection);
				}
			#endif

				// Specular profile
			#if SUBSTRATE_SPECPROFILE_ENABLED
				BRANCH
				if (BSDF_GETHASSPECPROFILE(BSDFContext.BSDF))
				{
					SubstrateEnvLightResult.SpecularWeight *= EvaluateSpecularProfileEnv(SLAB_SPECPROFILEID(BSDFContext.BSDF), BSDFContext.SatNoV, SubstrateEnvLightResult.SpecularSafeRoughness);
				}
			#endif

				SubstrateEnvLightResult.SSRSpecularWeight = SubstrateEnvLightResult.SpecularWeight;

				#if SUBSTRATE_FASTPATH==0
				if (bHasHaziness)
				{
					FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
					const float HazeWeight = Haziness.Weight;
					const float HazeSafeRoughness = MakeRoughnessSafe(Haziness.Roughness);
					const bool  bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;

					SubstrateEnvLightResult.SpecularHazeDirection = SubstrateGetOffSpecularPeakReflectionDir(ClearCoatTopContextN, ClearCoatTopContextR, HazeSafeRoughness);

					BRANCH
					if (bHazeAsSimpleClearCoat)
					{
						const float3 HazeClearCoatTransmittance = SimpleClearCoatTransmittance(ClearCoatBottomContextNoL, ClearCoatBottomContextNoV, SubstrateGetBSDFMetallic(BSDFContext.BSDF), SubstrateGetBSDFBaseColor(BSDFContext.BSDF));
						const float HazeF0 = 0.04f;
						const float HazeF90= 1.0f;
						FBxDFEnergyTermsRGB EnergyTerms = ComputeGGXSpecEnergyTermsRGB_Env(HazeSafeRoughness, ClearCoatTopContextNoV, HazeF0, HazeF90);
						const float3 EvalEnvBRDFHaze = EnergyTerms.E;
						const float3 TopLayerSpecularTransmittionApprox = saturate(1.0f - EvalEnvBRDFHaze); // Simple and do not use extra LUT for now

						const float TopLayerCoverage = HazeWeight;
						const float3 BottomLayerFactors = lerp(1.0, HazeClearCoatTransmittance * TopLayerSpecularTransmittionApprox, TopLayerCoverage);

						SubstrateEnvLightResult.SpecularWeight *= BottomLayerFactors;
						SubstrateEnvLightResult.DiffuseWeight  *= BottomLayerFactors;

						SubstrateEnvLightResult.SpecularHazeWeight = TopLayerCoverage * EvalEnvBRDFHaze;
						SubstrateEnvLightResult.SpecularHazeSafeRoughness = HazeSafeRoughness;

						SubstrateEnvLightResult.SSRSpecularWeight = lerp(SubstrateEnvLightResult.SpecularWeight, SubstrateEnvLightResult.SpecularHazeWeight, TopLayerCoverage);

						const float DirectionalAlbedo_SpecularHazeTransmission	= ComputeEnergyPreservation(EnergyTerms);
						DirectionalAlbedo_SpecularTransmission	= lerp(DirectionalAlbedo_SpecularTransmission, DirectionalAlbedo_SpecularHazeTransmission, TopLayerCoverage);
					}
					else
					{
						// Smoothly fade in haziness while fading out SSR to make sure we do not add energy and avoid popping.
						const float HazinessFadeIn = saturate(HazeWeight / 0.1f);
						const float HazeWeightWithFadeIn = lerp(0.0f, HazeWeight, HazinessFadeIn);
						SubstrateEnvLightResult.SSRReduction = HazeWeightWithFadeIn;

						// Apply blend factor on the sharp specular contribution
						const float BlendFactor = lerp(1.0f, (1.0f - HazeWeight), HazinessFadeIn);
						SubstrateEnvLightResult.SpecularWeight *= BlendFactor;
						SubstrateEnvLightResult.SSRSpecularWeight *= BlendFactor;

						// Compute the directional albedo for hazy specular
						FBxDFEnergyTermsRGB EnergyTerms = ComputeGGXSpecEnergyTermsRGB_Env(HazeSafeRoughness, BSDFContext.Context.NoV, F0, F90);
						const float3 EvalEnvBRDFHaze = EnergyTerms.E;
						SubstrateEnvLightResult.SpecularHazeWeight = lerp(0.0f, HazeWeight, HazinessFadeIn) * EvalEnvBRDFHaze;
						SubstrateEnvLightResult.SpecularHazeSafeRoughness = HazeSafeRoughness;

						const float DirectionalAlbedo_SpecularHazeTransmission	= ComputeEnergyPreservation(EnergyTerms);
						DirectionalAlbedo_SpecularTransmission	= lerp(DirectionalAlbedo_SpecularTransmission, DirectionalAlbedo_SpecularHazeTransmission, HazeWeightWithFadeIn); // Should not that be HazeWeight?
					}

					// Specular profile for haze
				#if SUBSTRATE_SPECPROFILE_ENABLED
					BRANCH
					if (BSDF_GETHASSPECPROFILE(BSDFContext.BSDF))
					{
						SubstrateEnvLightResult.SpecularHazeWeight *= EvaluateSpecularProfileEnv(SLAB_SPECPROFILEID(BSDFContext.BSDF), BSDFContext.SatNoV, SubstrateEnvLightResult.SpecularHazeSafeRoughness);
					}
				#endif

				}
				#endif // SUBSTRATE_FASTPATH==0

			} // if (bEnableSpecular)

			// Specular to diffuse energy preservation
			{
				SubstrateEnvLightResult.DiffuseWeight *= DirectionalAlbedo_SpecularTransmission;
			}

			#if SUBSTRATE_FASTPATH==0
			BRANCH
			if (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_WRAP || BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
			{
				const bool bTwoSided = BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_TWO_SIDED_WRAP;
				const float3 SlabDiffuseColor = bTwoSided ? DiffuseColor : float3(1, 1, 1);
				FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(SlabDiffuseColor, SLAB_SSSMFP(BSDFContext.BSDF));
			
				// We consider single scattering and perpendicular to the surface light transmittance.
				// Using BackFaceNoL instead of 1 for TransmittanceNoL would be more correct but it gives some high frequency details looking weird without multiple scattering.
				const float TransmittanceNoL = 1.0f;
				const float3 SlabTransmittance = IsotropicMediumSlabTransmittance(PM, SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, TransmittanceNoL);
			
				if (bTwoSided)
				{
					// We recover the transmitted luminance from as the back face Lambert affected by the transmittance.
					// The MFP is specified on the slab node and then transmittance recoved from it.
					// We assume the back face diffuse albedo is white=1 and no diffusion happens at this stage.
					const float3 BackFaceWhiteDiffuseColor = float3(1.0f, 1.0f, 1.0f);
			
					// The diffuse is attenuated by the slab transmittance and the GGX interface SpecularTransmission.
					SubstrateEnvLightResult.DiffuseBackFaceWeight = Diffuse_Lambert(BackFaceWhiteDiffuseColor) * SlabTransmittance * DirectionalAlbedo_SpecularTransmission;
				}
				else
				{
					// The wrap lighting is a non-physically based shading which intends to match legacy behavior.
					// We recover SubsurfaceColor that has been encoded as the Slab transmittance color.
					const float3 DirectionalSubSurfaceColor = SlabTransmittance * DirectionalAlbedo_SpecularTransmission;

					// As of today, for the diffuse to match the legacy behavior, we have to add the susurface color contribution.
					SubstrateEnvLightResult.DiffuseColor  += DirectionalSubSurfaceColor;
					SubstrateEnvLightResult.DiffuseWeight += DirectionalSubSurfaceColor;
				}
			}
			#endif

			#if SUBSTRATE_FASTPATH==0
			BRANCH
			if (BSDF_GETHASFUZZ(BSDFContext.BSDF))
			{
				const float  FuzzAmount = SLAB_FUZZ_AMOUNT(BSDFContext.BSDF);
				const float3 FuzzF0 = SLAB_FUZZ_COLOR(BSDFContext.BSDF);
				const float  FuzzRoughness = MakeRoughnessSafe(SLAB_FUZZ_ROUGHNESS(BSDFContext.BSDF), SUBSTRATE_MIN_FUZZ_ROUGHNESS);

				FBxDFEnergyTermsA ClothEnergyTerms = (FBxDFEnergyTermsA)1.0f;
				// Always use Sheen LTC as its cost is affordable and has better quality
				#if 1
				{
					const float DirectionalAlbedo = SheenLTC_DirectionalAlbedo(BSDFContext.Context.NoV, FuzzRoughness, View.SheenLTCTexture, View.SheenLTCSampler);
					ClothEnergyTerms.E = DirectionalAlbedo;
					ClothEnergyTerms.W = 1.f;
				}
				#else
				{
					// UE_DEPRECATED: Keep Charlie Sheen for now, but code will be removed in future release
					// F0 is ignored, as energy preservation is handle solely based on the white directional albedo and FuzzAmount
					ClothEnergyTerms = ComputeClothEnergyTermsA(FuzzRoughness, BSDFContext.Context.NoV);
				}
				#endif

				// The specular and diffuse components below the fuzz are only attenuated linearly according to the amount of fuzz.
				const float ClothEnegyPreservation = 1.f - ClothEnergyTerms.E;
				const float Cloth_DirectionalAlbedo_SpecularTransmission = lerp(1.0, ClothEnegyPreservation, FuzzAmount);

				SubstrateEnvLightResult.DiffuseColor		*= Cloth_DirectionalAlbedo_SpecularTransmission;
				SubstrateEnvLightResult.DiffuseWeight		*= Cloth_DirectionalAlbedo_SpecularTransmission;
				SubstrateEnvLightResult.SpecularWeight		*= Cloth_DirectionalAlbedo_SpecularTransmission;
				SubstrateEnvLightResult.SpecularHazeWeight	*= Cloth_DirectionalAlbedo_SpecularTransmission;
				SubstrateEnvLightResult.SSRSpecularWeight	*= Cloth_DirectionalAlbedo_SpecularTransmission;
				
				// We currently send the cloth environment lighting to be taken into account by the diffuse term.
				// SUBSTRATE_TODO: true a mix of diffuse and specular contribution according to roughness.
				SubstrateEnvLightResult.DiffuseColor		+= FuzzF0 * ClothEnergyTerms.E * FuzzAmount;
				SubstrateEnvLightResult.DiffuseWeight		+= FuzzF0 * ClothEnergyTerms.E * FuzzAmount;
				break;
			}
			#endif

			break;
		}

#if SUBSTRATE_FASTPATH==0

		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
			FGBufferData GBuffer	= (FGBufferData)0;
			GBuffer.BaseColor		= HAIR_BASECOLOR(BSDFContext.BSDF);
			GBuffer.Specular		= HAIR_SPECULAR(BSDFContext.BSDF);
			GBuffer.Roughness		= HAIR_ROUGHNESS(BSDFContext.BSDF);
			GBuffer.Metallic		= HAIR_SCATTER(BSDFContext.BSDF);
			GBuffer.CustomData.z	= HAIR_BACKLIT(BSDFContext.BSDF);
			GBuffer.ShadingModelID	= SHADINGMODELID_HAIR;
			GBuffer.WorldNormal		= BSDFContext.N;
			
			const float3 N = BSDFContext.N;
			const float3 V = BSDFContext.V;
			float3 L = 0;
			SubstrateEnvLightResult.DiffuseWeight = EvaluateEnvHair(GBuffer, V, N, L /*out*/);
			SubstrateEnvLightResult.DiffuseNormal = L;
			// No specular environment contribution as of today if not using the special HairStrand render path.
			// So no need to apply bForceFullyRough neither.
			break;
		}

		case SUBSTRATE_BSDF_TYPE_EYE:	
		{
			float3 DiffuseColor		= EYE_DIFFUSEALBEDO(BSDFContext.BSDF);
			float3 F0 				= EYE_F0(BSDFContext.BSDF);
			float3 F90 				= EYE_F90(BSDFContext.BSDF);

			// Specular occlusion is only used here once to affect the F90 source parameters. F0 will decrease naturally to 0.
			F90 *= F0RGBToMicroOcclusion(F0);

			if (Settings.bForceFullyRough)
			{
				// When rendering reflection captures, the BSDF roughness has already been forced to 1 using View.RoughnessOverrideParameter (see SubstrateSanitizeBSDF).
				EnvBRDFApproxFullyRough(DiffuseColor, F0, F90);
			}

			SubstrateEnvLightResult.DiffuseNormal			= BSDFContext.N;
			SubstrateEnvLightResult.bPostProcessSubsurface	= (BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION || BSDF_GETSSSTYPE(BSDFContext.BSDF) == SSS_TYPE_DIFFUSION_PROFILE) && SUBSTRATE_SSS_MATERIAL_OVERRIDE;

			float SafeRoughness = MakeRoughnessSafe(EYE_ROUGHNESS(BSDFContext.BSDF));

			float3 EvalEnvBRDF = 0;
			float3 DirectionalAlbedo_SpecularTransmission = 0;
			{
				FBxDFEnergyTermsRGB SpecularEnergyTerms = ComputeGGXSpecEnergyTermsRGB_Env(SafeRoughness, BSDFContext.Context.NoV, F0, F90);
				const float DiffuseDirectionalAlbedo = 1.0f;
				EvalEnvBRDF									= SpecularEnergyTerms.E;
				DirectionalAlbedo_SpecularTransmission		= ComputeEnergyPreservation(SpecularEnergyTerms);
				SubstrateEnvLightResult.DiffuseWeight		= DiffuseColor * DiffuseDirectionalAlbedo * DirectionalAlbedo_SpecularTransmission;
				SubstrateEnvLightResult.DiffuseColor		= DiffuseColor;
			}

			BRANCH
			if (bEnableSpecular)
			{
				SubstrateEnvLightResult.SpecularDirection		= SubstrateGetOffSpecularPeakReflectionDir(BSDFContext.N, BSDFContext.R, SafeRoughness);
				SubstrateEnvLightResult.SpecularWeight			= EvalEnvBRDF;
				SubstrateEnvLightResult.SpecularSafeRoughness	= SafeRoughness;
				SubstrateEnvLightResult.SpecularColor			= F0;
			}
			SubstrateEnvLightResult.SSRSpecularWeight = SubstrateEnvLightResult.SpecularWeight;

			break;
		}

		case SUBSTRATE_BSDF_TYPE_TOON:
		{
			const float3 DiffuseColor						= ComputeDiffuseAlbedo(TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float3 F0									= ComputeF0(TOON_SPECULAR(BSDFContext.BSDF), TOON_BASECOLOR(BSDFContext.BSDF), TOON_METALLIC(BSDFContext.BSDF));
			const float  F90								= 1.0f;
			const float SafeRoughness						= MakeRoughnessSafe(TOON_ROUGHNESS(BSDFContext.BSDF));

			const float ToonProfileId = TOON_PROFILEID(BSDFContext.BSDF);
			FToonProfileEvaluateSettings ToonEvalSettings = GetToonProfileEvaluateSettings(ToonProfileId);

		#if SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION
			SubstrateEnvLightResult.ToonProfileID			= TOON_PROFILEID(BSDFContext.BSDF);
		#endif

			float3 DiffEnvBRDF = 0.0f;
			float3 SpecEnvBRDF = 0.0f;
			float DirectionalAlbedo_SpecularTransmission = 1.0f;
			float3 MSScale = 1.0f;

		#if MOBILE_USE_PREINTEGRATED_GF
			SpecEnvBRDF = EnvBRDF(F0, SafeRoughness, BSDFContext.Context.NoV);
		#else
			SpecEnvBRDF = EnvBRDFApprox(F0, SafeRoughness, BSDFContext.Context.NoV);
		#endif
			DiffEnvBRDF = 1.0f;

			FToonProfileGISettings ToonProfileGISettings	= GetToonProfileGISettings(ToonProfileId);

			SubstrateEnvLightResult.DiffuseNormal			= BSDFContext.N;
			SubstrateEnvLightResult.bPostProcessSubsurface	= false;

			SubstrateEnvLightResult.DiffuseWeight			= DirectionalAlbedo_SpecularTransmission * DiffuseColor * DiffEnvBRDF * ToonProfileGISettings.DiffuseIndirectScale;
			SubstrateEnvLightResult.DiffuseColor			= DirectionalAlbedo_SpecularTransmission * DiffuseColor;

			SubstrateEnvLightResult.SpecularDirection		= SubstrateGetOffSpecularPeakReflectionDir(BSDFContext.N, BSDFContext.R, SafeRoughness);
			SubstrateEnvLightResult.SpecularWeight			= SpecEnvBRDF * ToonProfileGISettings.SpecularIndirectScale;
			SubstrateEnvLightResult.SpecularSafeRoughness	= SafeRoughness;
			SubstrateEnvLightResult.SpecularColor			= F0;
		}
		break;

		//case SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD:
		//case SUBSTRATE_BSDF_TYPE_UNLIT:
		//case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		//Nothing to do in this case because these BSDF are evaluated in other specialised passes.
		//break;
#endif
	}

	return SubstrateEnvLightResult;
}

FThreeBandSHVector SubstrateBSDFToSH(FSubstrateBSDFContext BSDFContext)
{
	FThreeBandSHVector SHVector;

	const uint BSDFType = BSDF_GETTYPE(BSDFContext.BSDF);
	if (BSDFType == SUBSTRATE_BSDF_TYPE_HAIR)
	{
		// Hack to avoid culling directions that hair will sample
		SHVector = (FThreeBandSHVector)0;
		SHVector.V0.x = 1.0f;
	}
	else
	{
		SHVector = CalcDiffuseTransferSH3(BSDFContext.N, 1.0f);
	}

	// SUBSTRATE_TODO adapt the SH to BSDFs
	return SHVector;
}

#endif // SUBSTRATE_ENABLED
