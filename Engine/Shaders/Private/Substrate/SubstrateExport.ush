// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../ToonProfileCommon.ush"

// Do not include that on mobile since FMobileBasePassInterpolantsVSToPS is not the same structure as FBasePassInterpolantsVSToPS
#if SUBSTRATE_ENABLED && !defined(FMobileBasePassInterpolantsVSToPS)

// This is used to evaluate some precomputed lighting data while we pack out Substrate material.
// It is also used to remove reference to unwanted resourced when not needed.
#ifndef MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
#error MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING must be defined before including SubstrateExport.ush
#endif

#ifndef SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
#error SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE must be defined from Substrate.ush before including SubstrateExport.ush
#endif

#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING

// Forward declaration
FShadingOcclusion ApplyBentNormal(
	in half3 CameraVector, 
	in half3 WorldNormal, 
	in float3 WorldBentNormal0, 
	in half Roughness,
	in half MaterialAO);

void GetPrecomputedIndirectLightingAndSkyLight(
	FMaterialPixelParameters MaterialParameters,
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FBasePassInterpolantsVSToPS BasePassInterpolants,
	VTPageTableResult LightmapVTPageTableResult,
	bool bEvaluateBackface,
	float3 DiffuseDir,
	float3 VolumetricLightmapBrickTextureUVs,
	out float3 OutDiffuseLighting,
	out float3 OutSubsurfaceLighting,
	out float OutIndirectIrradiance);

float3 SubstrateGetBSDFPrecomputedLighting(
	in FSubstrateIntegrationSettings Settings,
	in FSubstratePixelHeader SubstratePixelHeader,
	in FSubstrateBSDF CurrentBSDF,
	in float3 V,
	in float3 WorldBentNormal0,
	in FMaterialPixelParameters MaterialParameters,
	in FVertexFactoryInterpolantsVSToPS Interpolants,
	in FBasePassInterpolantsVSToPS BasePassInterpolants,
	in VTPageTableResult LightmapVTPageTableResult,
	in float3 VolumetricLightmapBrickTextureUVs,
	inout float2 SpecMultiBounceAO_IndirectIrradiance)
{
	FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;
	FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, V);

	// Evaluate shading occlusion data and bent normal
	FShadingOcclusion ShadingOcclusion = ApplyBentNormal(
		V,
		SubstrateBSDFContext.N,
		WorldBentNormal0,
		SubstrateGetBSDFRoughness(CurrentBSDF),
		SubstratePixelHeader.IrradianceAO.MaterialAO);

	// Update a context specific for environment lighting
	FSubstrateBSDFContext EnvBSDFContext = SubstrateBSDFContext;
	EnvBSDFContext.N = ShadingOcclusion.BentNormal;
	EnvBSDFContext.SubstrateUpdateBSDFContext(EnvBSDFContext.L);

	// For diffuse, we specify a perpendicular to the surface light direction for the transmittance to light to not be view dependent.
	float DiffuseEnvLightingNoL = 1.0f;
	float3 LuminanceWeightFinal = LuminanceWeight(DiffuseEnvLightingNoL, CurrentBSDF);

	// Evaluate material environment lighting parameters
	const bool bEnableSpecular = false;
	FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(EnvBSDFContext, bEnableSpecular, Settings);

	// Fetch precomputed lighting
	float3 DiffuseLighting = 0.0f;
	float3 SubsurfaceLighting = 0.0f;
	float  IndirectIrradiance = 0.0f;
	const bool bEvaluateBackface = any(SubstrateEnvLight.DiffuseBackFaceWeight > 0.0);	// With Substrate we have no other way to know if backface lighting will be needed
	GetPrecomputedIndirectLightingAndSkyLight(MaterialParameters, Interpolants, BasePassInterpolants, LightmapVTPageTableResult, bEvaluateBackface, SubstrateEnvLight.DiffuseNormal, VolumetricLightmapBrickTextureUVs,
		DiffuseLighting, SubsurfaceLighting, IndirectIrradiance);

	const float3 DiffMultiBounceAO = AOMultiBounce(SubstrateEnvLight.DiffuseColor, ShadingOcclusion.DiffOcclusion);
	SpecMultiBounceAO_IndirectIrradiance.x = AOMultiBounce(Luminance(SubstrateEnvLight.SpecularColor), ShadingOcclusion.SpecOcclusion).g;
	SpecMultiBounceAO_IndirectIrradiance.y = IndirectIrradiance;

	// Evaluate precomputed lighting according to material parameters matching environment lighting.
	float3 OutLuminance = (DiffuseLighting * SubstrateEnvLight.DiffuseWeight + SubsurfaceLighting * SubstrateEnvLight.DiffuseBackFaceWeight) * LuminanceWeightFinal * DiffMultiBounceAO;

	return OutLuminance;
}

#endif // MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING

// If this is changed, please update the compiler side material size evaluation in SubstrateMaterial.cpp
void PackSubstrateOut(
	inout FRWSubstrateMaterialContainer SubstrateBuffer,
#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	RWTexture2DArray<uint> ExtraMaterialDataUAV,
#endif
	float Dither,
	FSubstrateIntegrationSettings Settings,
	FSubstrateAddressing SubstrateAddressing,
	FSubstratePixelHeader SubstratePixelHeader,
	FSubstrateData Substrate,
	float3 V,
	float3 WorldBentNormal0,
	inout bool bSubstrateSubsurfaceEnable,
	inout float3 OutEmissiveLuminance,
	inout FSubstrateSubsurfaceData SSSData,
	inout FSubstrateTopLayerData TopLayerData,
	inout FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData
#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
	,in FMaterialPixelParameters MaterialParameters
	,in FVertexFactoryInterpolantsVSToPS Interpolants
	,in FBasePassInterpolantsVSToPS BasePassInterpolants
	,in VTPageTableResult LightmapVTPageTableResult
	,in float3 VolumetricLightmapBrickTextureUVs
	,inout float3 OutScatteredPrecomputedLuminance
#endif
)
{
	// This only exists with inline shading and when we are going to write out BSDFs (UpdateAllXXX functions needs to be defined)
#if SUBSTRATE_INLINE_SHADING && SUBSTRATE_CLAMPED_CLOSURE_COUNT > 0

	bSubstrateSubsurfaceEnable = false;
	OutEmissiveLuminance = 0.0f;
	#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
	OutScatteredPrecomputedLuminance = 0.0f;
	#endif

	const float FullyRough = 1.0f;

	// While packing Substrate layer data, Classification/SSS/TopLayer data are extracted & stored for dedicated passes
	// This avoid to run a post-basepass which would re-read all the material data
	SSSData = (FSubstrateSubsurfaceData)0;
	TopLayerData = (FSubstrateTopLayerData)0;
	OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;

	// The SSSData we accumulate to allow blending of Wrap and SubstrateDiffusion types.
	uint SSSDataType = SSS_TYPE_NONE;
	float SSSDataAniso = 0.0f;
	uint  SSSDataProfileID = 0;
	float SSSDataRadiusScale = 0.0f;
	float3 SSSDataMFP = 0.0f;
	float3 SSSDataBaseColor = 0.0f;

	uint BSDFVisibleCount = 0;
	
	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		// Update tree (coverage/transmittance/luminace weights)
		SubstratePixelHeader.SubstrateUpdateTree(V, Settings);

		const uint RootOperatorIndex = Substrate.OperatorIndex;

	#if SUBSTRATE_ADVANCED_DEBUG_ENABLED && SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
		if (Settings.SliceStoringDebugSubstrateTreeData > -1)
		{
			SubstratePackOutSubstrateTree(ExtraMaterialDataUAV, RootOperatorIndex, SubstratePixelHeader.SubstrateTree, Settings.SliceStoringDebugSubstrateTreeData);
		}
	#endif

		// Process the rough refraction data (this code is only used, and not culled out, if effectively used in base pass for instance)
		{
			const FSubstrateOperator RootOperator = SubstratePixelHeader.SubstrateTree.Operators[Substrate.OperatorIndex];
			OpaqueRoughRefractionData.Coverage = RootOperator.OpaqueRoughRefractCoverage;

			const float StandardDeviationCm = sqrt(GetVarianceFromRoughness(RootOperator.OpaqueRoughRefractTopRoughness));
			const float StandardDeviationCmForThickness = StandardDeviationCm * RootOperator.OpaqueRoughRefractThicknessCm;
			OpaqueRoughRefractionData.VarianceCm = StandardDeviationCmForThickness * StandardDeviationCmForThickness;
		}

		uint OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_NONE;
		bool bIsSimpleMaterial = true;
		bool bIsSingleMaterial = true;
		bool bIsOnlySlab = true;
		int OneBSDFMaterial_Index = 0; // Use for simple and single material modes. Represent the last visible BSDF index.
		float TopLayerTotalWeight = 0.0f;
		
		{
			Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
			{
			#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

				const bool bIsVisible = SubstrateIsBSDFVisible(BSDF);

				if (bIsVisible)
				{
					BSDFVisibleCount++;

					switch (BSDF_GETTYPE(BSDF))
					{
					case SUBSTRATE_BSDF_TYPE_SLAB:
					{
						OutEmissiveLuminance += BSDF_GETEMISSIVE(BSDF) * BSDF.LuminanceWeightV;

						// Simple and single bsdf do not allow weights other than 1
						const bool LuminanceWeightEqualOne = all(BSDF.LuminanceWeightV == 1.0f);

						#if SUBSTRATE_COMPLEXSPECIALPATH
						const bool bForceComplexSpecialPath = (SUBSTRATE_GLINTS_ENABLED && BSDF_GETHASGLINT(BSDF));
						#else
						const bool bForceComplexSpecialPath = false;
						#endif
						const bool bForceComplexMaterial = BSDF_GETHASANISOTROPY(BSDF) || (SUBSTRATE_SPECPROFILE_ENABLED && BSDF_GETHASSPECPROFILE(BSDF)) || bForceComplexSpecialPath || !LuminanceWeightEqualOne;

						// Update simple material compatibility
						bIsSimpleMaterial = bIsSimpleMaterial && IsSubstrateSlabCompatible_SimplePath(BSDF) && !bForceComplexMaterial;

						// Update single material compatibility. For now, single materials don't support anisotropy, 
						// they use the complex pass, as the toplayer data does not contain the full frame basis, only the top normal.
						bIsSingleMaterial = bIsSingleMaterial && !bForceComplexMaterial;

						TopLayerTotalWeight += BSDF.TopLayerDataWeight;
						float TopLayerRoughnessContribution = BSDF.TopLayerDataWeight * SLAB_ROUGHNESS(BSDF);
						float3 TopLayerBaseColorContribution = BSDF.TopLayerDataWeight * SubstrateGetBSDFBaseColor(BSDF);
						if (BSDF_GETHASHAZINESS(BSDF))
						{
							FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
							if (Haziness.bSimpleClearCoat)
							{
								// Transfert SSR from clear coat to bottom layer according to coverage.
								TopLayerRoughnessContribution = BSDF.TopLayerDataWeight * lerp(SLAB_ROUGHNESS(BSDF), Haziness.Roughness, Haziness.Weight);
								//TopLayerBaseColorContribution = BSDF.TopLayerDataWeight * lerp(0.0, 0.04f, F0RGBToMetallic(0.04f)); Skip because, as for the legacy path, we only show the bottom BaseColor of Clear Coat materials.
							}
						}
						TopLayerData.Roughness			+= TopLayerRoughnessContribution;
						TopLayerData.UnlitViewBaseColor += TopLayerBaseColorContribution;

						TopLayerData.Material = TOP_LAYER_MATERIAL_VALID;

						const bool bHasSSS = BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE;
						const bool bIsSimpleVolume = BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_SIMPLEVOLUME;
						bSubstrateSubsurfaceEnable = bSubstrateSubsurfaceEnable || (bHasSSS && BSDF.Coverage > 0.0f && !bIsSimpleVolume); // Should it be coverage>0 or any(LuminanceWeight>0) ?

						const bool bSSSMask = BSDF.bIsBottom;	// SSS and Thin can only be on the bottom layer, so ignoring bIsSimpleVolume here.
						SubstratePixelHeader.SetHasSubsurface(bSSSMask && bHasSSS && !bIsSimpleVolume);

						// Ensure the complex path mode is not overriden with a "Slab" if a previous BSDF set it to "SlabSpecial"
						#if SUBSTRATE_COMPLEXSPECIALPATH
						SubstratePixelHeader.SetComplexPathMode(bForceComplexSpecialPath ? COMPLEXPATH_MODE_SLABSPECIAL : SubstratePixelHeader.GetComplexPathMode());
						#else
						SubstratePixelHeader.SetComplexPathMode(COMPLEXPATH_MODE_SLAB);
						#endif
						break;
					}

					case SUBSTRATE_BSDF_TYPE_HAIR:
					{
						bIsOnlySlab = false;
						bIsSimpleMaterial = false;
						bIsSingleMaterial = false;
						OutEmissiveLuminance += BSDF_GETEMISSIVE(BSDF) * BSDF.LuminanceWeightV;

						TopLayerTotalWeight				+= BSDF.TopLayerDataWeight;
						TopLayerData.Roughness			+= BSDF.TopLayerDataWeight * FullyRough;
						TopLayerData.UnlitViewBaseColor += BSDF.TopLayerDataWeight * SubstrateGetBSDFBaseColor(BSDF);

						TopLayerData.Material = TOP_LAYER_MATERIAL_VALID;

						SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_HAIR);
						break;
					}

					case SUBSTRATE_BSDF_TYPE_EYE:
					{
						bIsOnlySlab = false;
						bIsSimpleMaterial = false;
						bIsSingleMaterial = false;

						TopLayerTotalWeight += BSDF.TopLayerDataWeight;
						TopLayerData.Roughness += BSDF.TopLayerDataWeight * FullyRough;

						TopLayerData.Material = TOP_LAYER_MATERIAL_VALID;

						const bool bHasSSS = BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE;
						SubstratePixelHeader.SetHasSubsurface(bHasSSS);
						SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_EYE);
						SubstratePixelHeader.SetCastContactShadow(false); // Eye shading disables contact shadow. See CastContactShadow(...)
						break;
					}

					case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
					{
						bIsOnlySlab = false;
						bIsSimpleMaterial = false;
						bIsSingleMaterial = false;
						OutEmissiveLuminance += BSDF_GETEMISSIVE(BSDF) * BSDF.LuminanceWeightV;

						TopLayerTotalWeight				+= BSDF.TopLayerDataWeight;
						TopLayerData.Roughness			+= BSDF.TopLayerDataWeight * SLW_ROUGHNESS(BSDF);
						TopLayerData.UnlitViewBaseColor += BSDF.TopLayerDataWeight * SubstrateGetBSDFBaseColor(BSDF);

						TopLayerData.Material = TOP_LAYER_MATERIAL_SLWATER;

						SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_SLWATER);
						break;
					}

					case SUBSTRATE_BSDF_TYPE_TOON:
					{
						bIsOnlySlab	= false;
						bIsSimpleMaterial = false;
						bIsSingleMaterial = false;
						OutEmissiveLuminance += BSDF_GETEMISSIVE(BSDF) * BSDF.LuminanceWeightV;

						TopLayerTotalWeight				+= BSDF.TopLayerDataWeight;
						TopLayerData.Roughness			+= BSDF.TopLayerDataWeight * TOON_ROUGHNESS(BSDF);
						TopLayerData.UnlitViewBaseColor += BSDF.TopLayerDataWeight * SubstrateGetBSDFBaseColor(BSDF);

						TopLayerData.Material = TOP_LAYER_MATERIAL_VALID;

						SubstratePixelHeader.SetHasSubsurface(true);									// This is true as Toon uses transmittance shadow. This also prevents classification pass to clear SSSHeader.
						SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_SLAB_COMPLEX);			// Use the complex path for Toon for now.
						SubstratePixelHeader.SetComplexPathMode(COMPLEXPATH_MODE_TOON);
						break;
					}

					}

					float3x3 TangentBasis = SubstrateGetBSDFSharedBasis_InlineShading(SubstratePixelHeader, BSDF);
					float3 N = TangentBasis[2];
					TopLayerData.WorldNormal += N * BSDF.TopLayerDataWeight;

					OneBSDFMaterial_Index = BSDFIdx;

					// Notify that the BSDF is at the top for SSR to only affect reflection there and not on the lower layers
					BSDF_SETISTOPLAYER(BSDF, BSDF.bIsTop ? 1 : 0);
				}
		
		
			#undef BSDF
			}
		}

		// Finalize top layer data
		TopLayerData.WorldNormal		= TopLayerTotalWeight > 0.0f	? normalize(TopLayerData.WorldNormal)					: 0.0f;
		TopLayerData.Roughness			= TopLayerTotalWeight > 0.0f	? TopLayerData.Roughness / TopLayerTotalWeight			: 0.0f;
		TopLayerData.UnlitViewBaseColor	= TopLayerTotalWeight > 0.0f	? TopLayerData.UnlitViewBaseColor / TopLayerTotalWeight	: 0.0f;

		// Set storage layout as either: fast(0), single(1), or complex(2)
		// We check BSDFVisibleCount to make sure we do not set a MATERIALMODE that would enforce a BSDFVisibleCount==1.
		if (bIsOnlySlab && BSDFVisibleCount > 0)
		{
			bIsSimpleMaterial = bIsSimpleMaterial && BSDFVisibleCount == 1;
			bIsSingleMaterial = bIsSingleMaterial && BSDFVisibleCount == 1;
			SubstratePixelHeader.SetMaterialMode(bIsSimpleMaterial ? HEADER_MATERIALMODE_SLAB_SIMPLE : bIsSingleMaterial ? HEADER_MATERIALMODE_SLAB_SINGLE : HEADER_MATERIALMODE_SLAB_COMPLEX);

			if (bIsSingleMaterial)
			{
			#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[0]
				const uint SingleHasHaziness		= 0x01;
				const uint SingleHasClearCoat		= 0x02;
				const uint SingleHasCloth			= 0x04;
				const uint SingleIsSimpleVolume		= 0x10;
				const uint SingleIsWrap				= 0x20;
				const uint SingleIsTwoSidedWrap		= 0x40;
				const uint SingleIsSSSProfile		= 0x80;
				// No need to add anisotropy since it is not supported by the simple mode.

				// If a surface has IsThin bit, opt-out legacy format.
				// Legacy mode doesn't need support for IsThin, and their storage format does not have space for storing IsThin bit. 
				const bool bIsThinSurface = BSDF_GETISTHIN(BSDF);

				uint OptimisedModeMask = 0;
				if (BSDF_GETHASHAZINESS(BSDF))
				{
					OptimisedModeMask |= SingleHasHaziness; // Regular haziness should disable non haziness/clearcoat fast path

					FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
					if (Haziness.bSimpleClearCoat)
					{
						OptimisedModeMask |= SingleHasClearCoat;
					}
				}
				if (BSDF_GETHASFUZZ(BSDF) && SLAB_ROUGHNESS(BSDF) == SLAB_FUZZ_ROUGHNESS(BSDF))
				{
					OptimisedModeMask |= SingleHasCloth;
				}
				if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_SIMPLEVOLUME)
				{
					OptimisedModeMask |= SingleIsSimpleVolume;
				}
				if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_WRAP)
				{
					OptimisedModeMask |= SingleIsWrap;
				}
				if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
				{
					OptimisedModeMask |= SingleIsTwoSidedWrap;
				}
				if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
				{
					OptimisedModeMask |= SingleIsSSSProfile;
				}

				if (((OptimisedModeMask & SingleHasClearCoat) == SingleHasClearCoat) || ((OptimisedModeMask & SingleHasClearCoat) == (SingleHasHaziness | SingleHasClearCoat)))
				{
					OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_CLEARCOAT;
				}
				else if ((OptimisedModeMask & SingleHasCloth) == SingleHasCloth && BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_NONE)
				{
					OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_CLOTH;
				}				
				else if ((OptimisedModeMask & SingleIsTwoSidedWrap) == SingleIsTwoSidedWrap && !BSDF_GETHASFUZZ(BSDF) && !bIsThinSurface)
				{
					OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP;
					SSSData.Header.Bytes = 0; // Force to skip SSSData writing
				}
				else if ((OptimisedModeMask & SingleIsWrap) == SingleIsWrap && !BSDF_GETHASFUZZ(BSDF) && !bIsThinSurface)
				{
					OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_SSSWRAP;
					SSSData.Header.Bytes = 0; // Force to skip SSSData writing
				}
				else if ((OptimisedModeMask & SingleIsSSSProfile) == SingleIsSSSProfile && !bIsThinSurface)
				{
					OptimisedLegacyMode = SINGLE_OPTLEGACYMODE_SSSPROFILE;
					SSSData.Header.Bytes = 0; // Force to skip SSSData writing
				}
			#undef BSDF
			}
		}

		// Now evaluate Specular Occlusion and Indirect Irradiance.
		#if MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING
		if(BSDFVisibleCount > 0)
		{
			float BSDFCount = 0;
			float2 SpecMultiBounceAO_IndirectIrradiance_Acc = 0.0f;
			Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
			{
			#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

				const bool bIsVisible = SubstrateIsBSDFVisible(BSDF);

				// Only write visible BSDF
				BRANCH
				if (bIsVisible)
				{
					// Convert BSDF to data ready to evaluate precomputed lighting
					float2 SpecMultiBounceAO_IndirectIrradiance = 0.0f;
					float3 PrecomputedLighting = SubstrateGetBSDFPrecomputedLighting(
						Settings,
						SubstratePixelHeader,
						BSDF,
						V,
						WorldBentNormal0,
						MaterialParameters,
						Interpolants,
						BasePassInterpolants,
						LightmapVTPageTableResult,
						VolumetricLightmapBrickTextureUVs,
						SpecMultiBounceAO_IndirectIrradiance);
					SpecMultiBounceAO_IndirectIrradiance_Acc += SpecMultiBounceAO_IndirectIrradiance;

					// Selectively add PrecomputedLighting to either OutScatteredPrecomputedLuminance or OutEmissiveLuminance 
					// depending if the light will be processed or not by SSS postprocess
					if (bSubstrateSubsurfaceEnable)
					{
						OutScatteredPrecomputedLuminance = PrecomputedLighting;
					}
					else
					{
						OutEmissiveLuminance += PrecomputedLighting;
					}

					// Stop writing if above budget.
					BSDFCount++;
				}

			#undef BSDF
			}

			// Write the combined SpecMultiBounceAO for all BSDFs as a single AO value. This is a trade off to have a mix of all the interactions, and it is correct for a single BSDF.
			float2 SpecMultiBounceAO_IndirectIrradiance = SpecMultiBounceAO_IndirectIrradiance_Acc * rcp(max(1.0f, BSDFCount));
			SubstratePixelHeader.IrradianceAO.MaterialAO			= SpecMultiBounceAO_IndirectIrradiance.x;
			SubstratePixelHeader.IrradianceAO.IndirectIrradiance	= SpecMultiBounceAO_IndirectIrradiance.y;
		}
		#endif
		
		// Store irradiance and AO into the state bits of the header so that they can be written later as part of the common header state bits.
	#if SUBSTRATE_INLINE_SHADING
		HEADER_SETIRRADIANCE_AO(SubstratePixelHeader.State, SubstratePackIrradianceAndOcclusion(SubstratePixelHeader.IrradianceAO, 0));
	#endif

		// Now write out Substrate data

		///////////////////////////////////////////////////////////////////////////
		// 3 types of encodings
		// * A. Layout0: Simple encoding	(use top layer normal)
		// * B. Layout1: Single encoding	(use top layer normal)
		// * C. Eye    : Custom encoding	(use top layer normal)
		// * D. Hair   : Custom encoding	(use top layer normal)
		// * E. Layout2: Complex encoding	(use basis)
		const bool bHasFastEncoding = SubstratePixelHeader.IsSimpleMaterial();
		const bool bHasFastWaterEncoding = SubstratePixelHeader.IsSingleLayerWater();
		const bool bHasSingleEncoding = SubstratePixelHeader.IsSingleMaterial();
		const bool bCustomEncoding = SubstratePixelHeader.IsHair() || SubstratePixelHeader.IsEye();

		// (Layout2)
		if (!bHasFastEncoding && !bHasSingleEncoding && !bCustomEncoding && !bHasFastWaterEncoding)
		{
			// 1. the header (Regular/Complex encoding)
			const uint PackedHeader = PackSubstrateHeader(BSDFVisibleCount, SubstratePixelHeader);
			SUBSTRATE_STORE_UINT1(PackedHeader);

			// 1.1 tangent basis (Regular/Complex encoding)
		#if SUBSTRATE_INLINE_SHADING
			UNROLL
			for (uint i = 0; i < SubstratePixelHeader.SharedLocalBases.Count; ++i)
			{
				const uint BasisType = SubstrateGetSharedLocalBasisType(SubstratePixelHeader.SharedLocalBases.Types, i);
				if (BasisType == SUBSTRATE_BASIS_TYPE_NORMAL)
				{
					SUBSTRATE_STORE_UINT1(SubstratePackNormal(SubstratePixelHeader.SharedLocalBases.Normals[i]));
				}
				else // if (BasisType == SUBSTRATE_BASIS_TYPE_TANGENT)
				{
					SUBSTRATE_STORE_UINT1(SubstratePackNormalAndTangent(SubstratePixelHeader.SharedLocalBases.Normals[i], SubstratePixelHeader.SharedLocalBases.Tangents[i]));
				}
			}
		#endif
		}

		{
			int BSDFCount = 0;
			Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
			{
			#define BSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

				const bool bIsVisible = SubstrateIsBSDFVisible(BSDF);

				// Only write visible BSDF
				BRANCH
				if (bIsVisible)
				{
					const uint GreyScaleThroughputV = SubstrateHasGreyScaleWeight(BSDF.LuminanceWeightV) ? 1 : 0;
					BSDF_SETHASGREYWEIGHT_V(BSDF, GreyScaleThroughputV);

					const bool bTransmittanceAboveAlongNRequired = any(BSDF.TransmittanceAboveAlongN < 1.0f);
					BSDF_SETHASTRANSABOVE(BSDF, bTransmittanceAboveAlongNRequired);

					// A. Layout0 - Simple encoding (aka. fast-path): store header & data
					const bool bFastEncodedBSDF = bHasFastEncoding && BSDFIdx == OneBSDFMaterial_Index;
					if (bFastEncodedBSDF)
					{
						uint Data0 = 0;
						uint Data1 = 0;
						PackDiffuseAlbedoF0Roughness(Dither, SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), SLAB_ROUGHNESS(BSDF), Data0, Data1);

						// Data0 (Header_State|Header_AO|Roughness|Diffuse8bits)
						{
							uint Out = 0;
							HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);

							SUBSTRATE_STORE_UINT1(Out | Data0);

						#if (HEADER_FASTENCODING_BIT_COUNT + 16) > 32
						#error Substrate fast path header is > 32bits
						#endif
						}
						// Data1
						{
							SUBSTRATE_STORE_UINT1(Data1);
						}
						// Ensure the rest of the BSDF is not stored with the regular path
						BSDF.State = 0;
					}
					// B. Layout1 - Simple encoding: single BSDF, whose header & BSDF state are merged
					else if (bHasSingleEncoding && BSDFIdx == OneBSDFMaterial_Index)
					{
						// Enforce BSDF to be flagged as 'top', since it is single-encoded (which ensure there is a single visible slab).
						// This is needed when material contains other slabs which are dynamically culled based on coverage
						BSDF_SETISTOPLAYER(BSDF, 1);

						// Data0 (Header_State|Header_AO|Header_BSDFTypes|BSDF_State)
						uint Out = 0;
						HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);

						Out = Out & HEADER_SINGLEENCODING_MASK;
						Out = Out | ((OptimisedLegacyMode & HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK)	<< (HEADER_SINGLEENCODING_BIT_COUNT));

						BRANCH
						if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_NONE)
						{
							Out = Out | ((BSDF.State & BSDF_SINGLEENCODING_MASK) << (HEADER_SINGLE_TOTAL_BIT_COUNT));
							SUBSTRATE_STORE_UINT1(Out);
						}
						// With this special path, there is 32-8-8-3= 13bits available for packing on the header.
						// Because we use 3 UINT MRT, this means that a total of 13+32+32 = 77 bits.
						else if(OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLEARCOAT)
						{
							uint Data0 = 0; 
							uint Data1 = 0;
							uint Data2 = SetHazinessRoughness(SLAB_HAZINESS(BSDF), SLAB_ROUGHNESS(BSDF));
							PackDiffuseAlbedoF0(Dither, SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), Data0, Data1);

							SUBSTRATE_STORE_UINT1(Out | Data0);
							SUBSTRATE_STORE_UINT1(Data1);
							SUBSTRATE_STORE_UINT1(Data2);

							// Ensure the rest of the BSDF is not stored with the regular path
							BSDF.State = 0;
						}
						else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_CLOTH)
						{
							uint Data0 = 0; 
							uint Data1 = 0;
							uint Data2 = PackRGBA8(float4(SLAB_FUZZ_COLOR(BSDF), SLAB_FUZZ_AMOUNT(BSDF)));
							PackDiffuseAlbedoF0(Dither, SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), Data0, Data1);

							SUBSTRATE_STORE_UINT1(Out | Data0);
							SUBSTRATE_STORE_UINT1(Data1);
							SUBSTRATE_STORE_UINT1(Data2);

							// Ensure the rest of the BSDF is not stored with the regular path
							BSDF.State = 0;
						}
						else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSWRAP || OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP)
						{
							const uint PackedSSSWOpacity7bits = PackR7(SubstrateSubSurfaceGetWrapOpacityFromAnisotropy(SLAB_SSSPHASEANISOTROPY(BSDF)), 0.5);	// We should not dither here because it would have a large scale effect.

							uint Data0 = 0; 
							uint Data1 = 0;
							uint Data2 = PackR10G10B10F(SLAB_SSSMFP(BSDF));
							PackDiffuseAlbedoF0(Dither, SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), Data0, Data1);

							SUBSTRATE_STORE_UINT1(Out | Data0 | (BitFieldExtractU32(PackedSSSWOpacity7bits, 5, 0)<<(HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT)));
							SUBSTRATE_STORE_UINT1(Data1);
							SUBSTRATE_STORE_UINT1(Data2 | (BitFieldExtractU32(PackedSSSWOpacity7bits, 2, 5)<<30) );

							// Ensure the rest of the BSDF is not stored with the regular path
							BSDF.State = 0;
						}
						else if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSPROFILE)
						{
							uint Data0 = 0; 
							uint Data1 = 0;
							uint Data2 = (PackR8(SLAB_SSSPROFILERADIUSSCALE(BSDF))<<8) | (uint(SLAB_SSSPROFILEID(BSDF)) & 0xFF) | ((SLAB_HAZINESS(BSDF) & 0xFFFF)<<16);
							PackDiffuseAlbedoF0(Dither, SLAB_DIFFUSEALBEDO(BSDF), SLAB_F0(BSDF), Data0, Data1);

							SUBSTRATE_STORE_UINT1(Out | Data0);
							SUBSTRATE_STORE_UINT1(Data1);
							SUBSTRATE_STORE_UINT1(Data2);

							// Ensure the rest of the BSDF is not stored with the regular path
							BSDF.State = 0;
						}

						#if (HEADER_SIMPLEENCODING_BIT_COUNT) > 32
						#error Substrate fast path header is > 32bits
						#endif
					}
					// C. Layout X (Eye)
					else if (SubstratePixelHeader.IsEye())
					{
						// Data0 (Header_State|Header_AO|Data)
						uint Out = 0;
						HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);
						Out = Out & HEADER_EYEENCODING_MASK;
						Out = Out | PackRGBA8(float4(0, 0, EYE_IRISMASK(BSDF), EYE_IRISDISTANCE(BSDF)));
						SUBSTRATE_STORE_UINT1(Out);

						#if (HEADER_EYEENCODING_BIT_COUNT) > 32
						#error Substrate eye path header is > 32bits
						#endif
					}
					// D. Layout X (Hair)
					else if (SubstratePixelHeader.IsHair())
					{
						// Data0 (Header_State|Header_AO|Data)
						uint Out = 0;
						HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);
						Out = Out & HEADER_HAIRENCODING_MASK;
						SUBSTRATE_STORE_UINT1(Out);

						#if (HEADER_HAIRENCODING_BIT_COUNT) > 32
						#error Substrate hair path header is > 32bits
						#endif
					}
					// E. Layout X (SingleLayerWater)
					else if (bHasFastWaterEncoding)
					{
						#if (HEADER_SLWENCODING_BIT_COUNT) > 16
						#error Substrate single layer water header is > 16bits
						#endif

						// Now we pack diffuse and F0 in a special way: both encoded as R7G7B6 with gamma 2.0. The low bit cound will be hidden by dithering and TAA.
						// We pack 8 of the 32 bits in the first header uint and the remaining bits are pack is the lowest significant bit of the second uint.
						const uint PackedBaseColor20Bits = PackR7G7B6Gamma2(SLW_BASECOLOR(BSDF), Dither);
						const uint PackedBaseColor12Bits = PackedBaseColor20Bits & 0xFFF;
						const uint PackedBaseColor8Bits = (PackedBaseColor20Bits >> 12) & 0xFF;

						// Packed into the header
						const uint Roughness8Bits = PackR8(SLW_ROUGHNESS(BSDF));
						const uint PackedOpacityMetalSpec20Bits = PackR7G7B6Linear(float3(SLW_TOPMATERIALOPACITY(BSDF), SLW_METALLIC(BSDF), SLW_SPECULAR(BSDF)), Dither);

						// UINT0 (Header_State|Header_AO|BaseColor8bits|Roughness)
						{
							uint Out = 0;
							HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);

							uint Packed16 = (PackedBaseColor8Bits << 8) | Roughness8Bits;
							Out = (Out & HEADER_SLWENCODING_MASK) | (Packed16 << HEADER_SLWENCODING_BIT_COUNT);
							SUBSTRATE_STORE_UINT1(Out);
						}
						// UINT1 (BaseColor12bits|Opacity|Metallic|Specular)
						{
							uint Out = (PackedBaseColor12Bits << 20) | PackedOpacityMetalSpec20Bits;
							SUBSTRATE_STORE_UINT1(Out);
						}
						// Ensure the rest of the BSDF is not stored with the regular path
						BSDF.State = 0;

					}
					// F. Layout2 - Weight for Regular/Complex path
					else if (GreyScaleThroughputV > 0)
					{
						BSDF_SETWEIGHT10F(BSDF, Pack10F(BSDF.LuminanceWeightV.x));
						SUBSTRATE_STORE_UINT1(BSDF.State);
					}
					else
					{
						SUBSTRATE_STORE_UINT1(BSDF.State);
						SUBSTRATE_STORE_UINT1(PackR11G11B10F(BSDF.LuminanceWeightV));
					}

					// Layout1 & Layout2
					if (!bFastEncodedBSDF && !bHasFastWaterEncoding && OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_NONE)
					{
						const uint BSDFType = BSDF_GETTYPE(BSDF);
						switch (BSDF_GETTYPE(BSDF))
						{
						case SUBSTRATE_BSDF_TYPE_SLAB:
						{
							// DiffuseAlbedo / F0 / Roughness / Aniso / SSSAniso
							{
								uint Data1 = 0; 
								uint Data2 = 0;
								BSDF.PackSingleOrComplexPathSlabBSDF(Dither, Data1, Data2);
								SUBSTRATE_STORE_UINT1(Data1);
								SUBSTRATE_STORE_UINT1(Data2);
							}

							const bool bHasF90 = BSDF_GETHASF90(BSDF);
							if (bHasF90 || BSDF_GETHASHAZINESS(BSDF))
							{
								BRANCH
								if (bHasF90)
								{
									// What is important is to maintain the hue and saturation, so we scale the color by the maximum of its components
									float3 F90 = saturate(SLAB_F90(BSDF));
									const float Divisor = max(F90.r, max(F90.g, F90.b));
									F90 = Divisor > 0.0f ? F90 / Divisor : 1.0f;
									float3 F90YCoCg = LinearRGB_2_NormalisedYCoCg(F90);

									uint F90Data = PackRGBA8(float4(F90YCoCg.y, F90YCoCg.z, 0, 0));
									uint HazinessData = SLAB_HAZINESS(BSDF);
									SUBSTRATE_STORE_UINT1(HazinessData << 16 | F90Data);
								}
								else
								{
									// We only need to render clear coat material with bottom normal when converting legacy materials.
									// That can be a single material or a complex material such as when anisotropy is enabled on legacy material conversion.
									// In this case we must write the 32 bits of HAZINESS to get the bottom normal.
									// And that is fine because for legacy materials F90 is not allowed.
									uint HazinessData = SLAB_HAZINESS(BSDF);
									SUBSTRATE_STORE_UINT1(HazinessData);
								}
							}

							if (BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE)
							{
								// All SSS types store side data (SSSHeader/SSSData), apart from SIMPLEVOLUME
								if (BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_SIMPLEVOLUME)
								{
									// Always override the type, we can only blend similar types of subsurface lighting effects.
									SSSDataType = BSDF_GETSSSTYPE(BSDF);

									// Now we are going to blend all the SSS attributes in order to be able to lerp legacy Subsurface and SubstrateDiffusion.
									// Legacy SSSProfile ID is still set by the last encountered Slab.
									// Attributes are blended using opacity only, no need to use the throughput to the view (SSS post process wants raw source data like BaseColor and the BSDF will otherwise run the correct lighting math).

									if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_WRAP || BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_TWO_SIDED_WRAP)
									{
										SSSDataAniso += SLAB_SSSPHASEANISOTROPY(BSDF) * BSDF.Coverage;
									}
									else if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
									{
										SSSDataProfileID = SLAB_SSSPROFILEID(BSDF);
										SSSDataRadiusScale = SLAB_SSSPROFILERADIUSSCALE(BSDF);
									}
									else
									{
										SSSDataMFP += SLAB_SSSMFP(BSDF) * BSDF.Coverage;
									}

									SSSDataBaseColor += SLAB_DIFFUSEALBEDO(BSDF) * BSDF.Coverage;
								}

								if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE)
								{
									// Simple volume overrides SSS profile because it takes over for when not at the bottom of the BSDF layer, or during forward rendering.
									// Note: Since it is a profile, we can't rescale its MFP value. So we store the 'thin' thickness value for computing the correct transmission during evaluation
									SUBSTRATE_STORE_UINT1(PackSSSProfile(	SLAB_SSSPROFILEID(BSDF), SLAB_SSSPROFILERADIUSSCALE(BSDF), BSDF_GETTHICKNESSCM(BSDF)));
								}
								else
								{
									// If using non-profile SSS diffusion, rescale SSS so that we take into account the slab thickness. 
									// Note: This is done 'after' storing the SSSHeader/SSSData, as the original MFP needs to be preserved for correct post-process diffusion.
									//       This rescaled MFP value is only used for transmission evaluation, not the diffusion
									if (BSDF_GETISTHIN(BSDF) && BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION)
									{
										SLAB_SSSMFP(BSDF)= RescaleMFPToComputationSpace(SLAB_SSSMFP(BSDF), BSDF_GETTHICKNESSCM(BSDF), SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM);
									}

									// Path used for bottom most layer SSS, simple volume and two sided lighting.
									SUBSTRATE_STORE_UINT1(PackR11G11B10F(SLAB_SSSMFP(BSDF)));
								}
							}

							if (BSDF_GETHASFUZZ(BSDF))
							{
								SUBSTRATE_STORE_UINT1(PackFuzz(SLAB_FUZZ_COLOR(BSDF), SLAB_FUZZ_AMOUNT(BSDF), SLAB_FUZZ_ROUGHNESS(BSDF), Dither));
							}

							if (BSDF_GETHASTRANSABOVE(BSDF))
							{
								SUBSTRATE_STORE_UINT1(PackColorLinearToGamma2AlphaLinear(float4(BSDF.TransmittanceAboveAlongN, BSDF.CoverageAboveAlongN)));
							}

							#if SUBSTRATE_GLINTS_ENABLED
							if (BSDF_GETHASGLINT(BSDF))
							{
								const uint2 PackedGlintData = PackGlints(SLAB_GLINT_VALUE(BSDF), SLAB_GLINT_UV(BSDF));
								SUBSTRATE_STORE_UINT1(PackedGlintData.x);
								SUBSTRATE_STORE_UINT1(PackedGlintData.y);

								// We have to store derivative when rasterizing the mesh. 
								// That cannot be reconstructed in a deferred fashion (e.g. edge of a mesh overlaping with backgroud would look wrongly aliased)
								SUBSTRATE_STORE_UINT1(PackFloat2ToUInt(SLAB_GLINT_UVDDX(BSDF)));
								SUBSTRATE_STORE_UINT1(PackFloat2ToUInt(SLAB_GLINT_UVDDY(BSDF)));
							}
							#endif // SUBSTRATE_GLINTS_ENABLED
							#if SUBSTRATE_SPECPROFILE_ENABLED
							if (BSDF_GETHASSPECPROFILE(BSDF))
							{
								SUBSTRATE_STORE_UINT1(PackSpecularProfile(SLAB_SPECPROFILEID(BSDF)));
							}
							#endif // SUBSTRATE_SPECPROFILE_ENABLED

							// 8-28 bytes
						}
						break;
						case SUBSTRATE_BSDF_TYPE_HAIR:
						{
							SUBSTRATE_STORE_UINT1(PackColorLinearToGamma2AlphaLinear(float4(HAIR_BASECOLOR(BSDF), HAIR_ROUGHNESS(BSDF))));
							SUBSTRATE_STORE_UINT1(PackRGBA8(float4(HAIR_SCATTER(BSDF), HAIR_SPECULAR(BSDF), HAIR_BACKLIT(BSDF), HAIR_COMPLEXTRANSMITTANCE(BSDF))));
							// 8 bytes
						}
						break;
						case SUBSTRATE_BSDF_TYPE_EYE:
						{
							const float2 EncodedIrisNormal = UnitVectorToOctahedron(EYE_IRISNORMAL(BSDF));
							const float2 EncodedIrisPlaneNormal = UnitVectorToOctahedron(EYE_IRISPLANENORMAL(BSDF));
							SUBSTRATE_STORE_UINT1(PackColorLinearToGamma2AlphaLinear(float4(EYE_DIFFUSEALBEDO(BSDF), EYE_ROUGHNESS(BSDF))));
							SUBSTRATE_STORE_UINT1(PackRGBA8(float4(EncodedIrisNormal * 0.5f + 0.5f, EncodedIrisPlaneNormal * 0.5f + 0.5f)));
							// 8 bytes

							// Note: we don't store the SSS profile ID into the Eye data, as it is not needed there. The profile is only stored into the SSS data.
							const bool bHasSSS = BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_DIFFUSION_PROFILE;
							if (bHasSSS)
							{
								SSSDataType = SSS_TYPE_DIFFUSION_PROFILE;
								SSSDataProfileID = EYE_SSSPROFILEID(BSDF);
								SSSDataRadiusScale = 1.0f;
								SSSDataBaseColor = EYE_DIFFUSEALBEDO(BSDF) * BSDF.Coverage;
							}
						}
						break;
						case SUBSTRATE_BSDF_TYPE_TOON:
						{
							uint Data0 = 0;
							Data0 |= PackR6G6B5Gamma2(TOON_BASECOLOR(BSDF), Dither);
							Data0 |= PackR5(TOON_SPECULAR(BSDF),  Dither) << 17;
							Data0 |= PackR5(TOON_METALLIC(BSDF),  Dither) << 22;
							Data0 |= PackR5(TOON_ROUGHNESS(BSDF), Dither) << 27;
							SUBSTRATE_STORE_UINT1(Data0);

							uint Data1 = PackToonCustomData(TOON_PATTERNUVS(BSDF), TOON_PROFILEID(BSDF), Dither);
							SUBSTRATE_STORE_UINT1(Data1);

							if (BSDF_GETHASANISOTROPY(BSDF))
							{
								SUBSTRATE_STORE_UINT1(asuint(TOON_ANISOTROPY(BSDF)));
							}

							SSSDataType = SSS_TYPE_TRANSMITTANCE_SHADOW;

							const float ShadowMFPInMeters = GetToonProfileShadowSettings(TOON_PROFILEID(BSDF)).ShadowMFPInMeters;
							SSSDataMFP = ShadowMFPInMeters.xxx;
							// 12 to 16 bytes
						}
						break;
						//case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER: 
						//{
						//	// SLW always use the path defined by bHasFastWaterEncoding.
						//}
						//break;
						}
					}

					// Stop writing if above budget.
					BSDFCount++;
				}

			#undef BSDF
			}

			checkSlow(BSDFCount == BSDFVisibleCount);
		}

		FinalizeWrites(SubstrateBuffer, SubstrateAddressing);

		// Now generate the final SSSData structure from the parameter blended attributes
		if (SSSDataType != SSS_TYPE_NONE)
		{
			// All SSS types store side data (SSSHeader/SSSData), apart from SIMPLEVOLUME
			if (SSSDataType != SSS_TYPE_SIMPLEVOLUME)
			{
				// Always override the type, we can only blend similar types of subsrface
				SubstrateSubSurfaceHeaderSetSSSType(SSSData.Header, SSSDataType);

				if (SSSDataType == SSS_TYPE_WRAP || SSSDataType == SSS_TYPE_TWO_SIDED_WRAP)
				{
					SubstrateSubSurfaceHeaderSetWrap(SSSData.Header, SSSDataAniso);
				}
				else if (SSSDataType == SSS_TYPE_DIFFUSION_PROFILE)
				{
					SubstrateSubSurfaceHeaderSetProfile(SSSData.Header, SSSDataRadiusScale, SSSDataProfileID);
				}
				else if (SSSDataType == SSS_TYPE_TRANSMITTANCE_SHADOW)
				{
					SubstrateSubSurfaceHeaderSetTransmittanceShadowMFPInMeters(SSSData.Header, SSSDataMFP);
				}
				else
				{
					SubstrateSubSurfaceHeaderSetNonProfile(SSSData.Header, SSSDataMFP);
				}

				SubstrateSubsurfaceExtrasSetBaseColor(SSSData.Extras, SSSDataBaseColor);
			}
		}
	}
#endif // SUBSTRATE_INLINE_SHADING && SUBSTRATE_CLAMPED_CLOSURE_COUNT > 0

}

// Trimmed version of PackSubstrateOut() specialized for hair data writting. This is used by hair strands composition. 
// todo: This should be refactored to avoid duplicate code.
void WriteHair(
	FSubstratePixelHeader SubstratePixelHeader,
	FSubstrateAddressing SubstrateAddressing,
	FSubstrateBSDF BSDF,
	bool bWriteHairData,
	inout FSubstrateTopLayerData TopLayerData,
	inout FRWSubstrateMaterialContainer SubstrateBuffer,
	RWTexture2DArray<uint> ExtraMaterialDataUAV)
{
	// Top layer data
	{
		const float3x3 TangentBasis = SubstrateGetBSDFSharedBasis_InlineShading(SubstratePixelHeader, BSDF);
		const float3 N = TangentBasis[2];

		TopLayerData = (FSubstrateTopLayerData)0;
		TopLayerData.Material = TOP_LAYER_MATERIAL_VALID;
		TopLayerData.WorldNormal = N;
		TopLayerData.Roughness = HAIR_ROUGHNESS(BSDF);
	}
	// Header
	{
		SubstratePixelHeader.SetHasSubsurface(false);
		SubstratePixelHeader.SetMaterialMode(HEADER_MATERIALMODE_HAIR);
		SubstratePixelHeader.ClosureCount = 1;
	}
	{
		// Data0 (Header_State|Header_AO|Data)
		uint Out = 0;
		HEADER_SETCOMMONSTATES(Out, SubstratePixelHeader.State);
		Out = Out & HEADER_HAIRENCODING_MASK;
		SUBSTRATE_STORE_UINT1(Out);

		#if (HEADER_HAIRENCODING_BIT_COUNT) > 32
		#error Substrate hair path header is > 32bits
		#endif
	}
	// Data
	if (bWriteHairData)
	{
		SUBSTRATE_STORE_UINT1(PackColorLinearToGamma2AlphaLinear(float4(HAIR_BASECOLOR(BSDF), HAIR_ROUGHNESS(BSDF))));
		SUBSTRATE_STORE_UINT1(PackRGBA8(float4(HAIR_SCATTER(BSDF), HAIR_SPECULAR(BSDF), HAIR_BACKLIT(BSDF), HAIR_COMPLEXTRANSMITTANCE(BSDF))));
		// 8 bytes
	}
}

#endif // SUBSTRATE_ENABLED && !defined(FMobileBasePassInterpolantsVSToPS)

#ifndef SUBSTRATE_GPU_LIGHTMASS
#define SUBSTRATE_GPU_LIGHTMASS 0
#endif

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE && (SUBSTRATE_MATERIAL_EXPORT_TYPE > 0 || SUBSTRATE_GPU_LIGHTMASS > 0 || SUBSTRATE_GBUFFER_FORMAT==0 || defined(SUBSTRATE_MATERIAL_EXPORT_REQUESTED))

#ifndef MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
#define MATERIAL_IS_SUBSTRATEHIDDENCONVERSION 0
#endif
#ifndef SUBSTRATE_OPAQUE_DEFERRED
#define SUBSTRATE_OPAQUE_DEFERRED 0
#endif

// Hidden conversion from root node for opaque material guarantee that there is a BSDF output with Coverage=1 and Transmittance=0 that is always visible.
// SUBSTRATE_TODO a simple more general way to make this work would be to make sure there is no CoverageWeight node in the graph instead of MATERIAL_IS_SUBSTRATEHIDDENCONVERSION? In this case that optimisation would be valid too.
#define SUBSTRATE_OPTIMISED_LEGACY_OPAQUE_EXPORT (SUBSTRATE_OPAQUE_DEFERRED && MATERIAL_IS_SUBSTRATEHIDDENCONVERSION)

struct FExportResult
{
	float  Coverage;
	float3 TransmittancePreCoverage;
	float3 BaseColorPostCoverage;
	float3 BaseColor;
	float3 WorldNormal;					// Already normalized
	float3 EmissiveLuminance;
	float  Specular;
	float  Roughness;
	float  Anisotropy;
	float  Metallic;
	float4 CustomData;
	float3 SubsurfaceColor;
	float  SubsurfaceProfileIDFloat;
	float  SubsurfaceProfileRadiusScale;
	float3 WorldTangent;				// Already normalized
	float  ShadingModelID;
	float IndirectIrradianceScale;
	float Curvature;
	float Opacity;
#if USE_DEVELOPMENT_SHADERS
	float3 UnlitViewBaseColor;
#endif
};

// This is used to export very simple data for lightmaps, material baking or preview on editor node.
// In this case we force full simplification and only recover a single BSDF in the end/ 
FExportResult SubstrateMaterialExportOut(
	in FSubstrateIntegrationSettings Settings,
	in FSubstratePixelHeader SubstratePixelHeader,
	in FSubstrateData SubstrateData,
	in float3 V,
	in float3 SurfaceWorldNormal,
	in float3 WorldPosition_CamRelative,
	in float MobileShadingPathCurvature,
	in float Dither = 0.5f)
{
	FExportResult Out = (FExportResult)0;
	Out.IndirectIrradianceScale = 1.0;
	Out.SubsurfaceProfileRadiusScale = 1.0;

	const int BSDFIdx = 0;
	#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

	#if !SUBSTRATE_OPTIMISED_LEGACY_OPAQUE_EXPORT
	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	#endif
	{
		#if SUBSTRATE_OPTIMISED_LEGACY_OPAQUE_EXPORT
		// In this case, there is always a BSDF and it is always 100% visible.
		// Sanitize BSDF before it is exported.
		CurrentBSDF.SubstrateSanitizeBSDF();
		CurrentBSDF.LuminanceWeightV = 1.0f; 

		Out.Coverage = 1.f;
		Out.TransmittancePreCoverage = 1.f;
		#else
		// Update tree (coverage/transmittance/luminace weights) and Sanitize its content.
		SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V, Settings, Out.Coverage, Out.TransmittancePreCoverage);
		#endif

		#if !SUBSTRATE_OPTIMISED_LEGACY_OPAQUE_EXPORT
		if (SubstrateIsBSDFVisible(CurrentBSDF))
		#endif
		{
			const uint BSDFType = BSDF_GETTYPE(CurrentBSDF);
			const uint SSSType  = BSDF_GETSSSTYPE(CurrentBSDF);

			const float3x3 TangentBasis = SubstrateGetBSDFSharedBasis_InlineShading(SubstratePixelHeader, CurrentBSDF);
			const float3 N = TangentBasis[2];
			const float3 T = TangentBasis[0];

			const float GreyScaleWeight = dot(CurrentBSDF.LuminanceWeightV, (1.0 / 3.0).xxx);
			const float NoV = saturate(dot(SurfaceWorldNormal, V));

			// Bake haziness second specular lobe with its own roughness into primary specular lobe (if not simple clear coat legacy shading model).
			CurrentBSDF.SubstrateBakeHazinessToSingleRoughness();

			// Bake the F90 into F0 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
			CurrentBSDF.SubstrateBakeF90ToReflectivity(NoV);
			BSDF_SETHASF90(CurrentBSDF, 0);
			
			// Bake the Specular Profiles into F0/F90 of BSDFs to blend in order to retain some of the original look when using parameter blending or fully simplified materials.
		#if SUBSTRATE_SPECPROFILE_ENABLED && MATERIAL_USES_SPECULAR_PROFILE
			CurrentBSDF.SubstrateBakeSpecularProfileToReflectivity(NoV);
			BSDF_SETHASSPECPROFILE(CurrentBSDF, 0);
		#endif

			Out.BaseColorPostCoverage	= SubstrateGetBSDFBaseColor(CurrentBSDF)	* CurrentBSDF.LuminanceWeightV;
			Out.BaseColor				= SubstrateGetBSDFBaseColor(CurrentBSDF);
			Out.WorldNormal				= N;
			Out.EmissiveLuminance		= BSDF_GETEMISSIVE(CurrentBSDF)				* CurrentBSDF.LuminanceWeightV;
			Out.Specular				= SubstrateGetBSDFSpecular(CurrentBSDF);
			Out.Roughness				= SubstrateGetBSDFRoughness(CurrentBSDF);	// When using the adaptive GBuffer, this exported roughness can be roughness modified by the dual specular from a SubsurfaceProfile shading model.
			Out.Anisotropy				= SubstrateGetBSDFAnisotropy(CurrentBSDF);
			Out.Metallic				= SubstrateGetBSDFMetallic(CurrentBSDF);
			Out.Opacity 				= SubstrateGetBSDFOpacity(CurrentBSDF);
			Out.SubsurfaceColor			= SubstrateGetBSDFSubSurfaceColor(CurrentBSDF);
			Out.SubsurfaceProfileIDFloat= ConvertSubsurfaceProfileToFloat(SubstrateGetBSDFSubSurfaceProfileID(CurrentBSDF));
			Out.WorldTangent			= T;
			Out.ShadingModelID			= SubstrateGetLegacyShadingModels(CurrentBSDF);
			
		#if USE_DEVELOPMENT_SHADERS
			Out.UnlitViewBaseColor		= CurrentBSDF.TopLayerDataWeight * SubstrateGetBSDFBaseColor(CurrentBSDF);
		#endif

#if MATERIAL_SHADINGMODEL_HAIR || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION // Hair shading model or if not a hidden conversion, then we cannot fully trust the shading model from simplified substrate nodes.
			if (BSDFType == SUBSTRATE_BSDF_TYPE_HAIR)
			{
				Out.Metallic	= HAIR_SCATTER(CurrentBSDF);
				Out.CustomData  = float4(0.0, 0.0, HAIR_BACKLIT(CurrentBSDF), 0.0);
			}
			else 
#endif
#if MATERIAL_SHADINGMODEL_EYE || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (BSDFType == SUBSTRATE_BSDF_TYPE_EYE)
			{
				const float3 IrisNormal		= EYE_IRISNORMAL(CurrentBSDF);
				const float3 IrisPlaneNormal= EYE_IRISPLANENORMAL(CurrentBSDF);
				const float IrisMask		= saturate(EYE_IRISMASK(CurrentBSDF));
				const float IrisDistance	= saturate(EYE_IRISDISTANCE(CurrentBSDF));

				Out.CustomData.x = EncodeSubsurfaceProfile(Out.SubsurfaceProfileIDFloat).x;
				Out.CustomData.w = 1.0f - IrisMask;	// Opacity

				const float3 GBufferWorldNormal = N;
			#if IRIS_NORMAL
				float2 WorldNormalOct = UnitVectorToOctahedron(GBufferWorldNormal);

				// CausticNormal stored as octahedron
				#if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
					// Blend in the negative intersection normal to create some concavity
					// Not great as it ties the concavity to the convexity of the cornea surface
					// No good justification for that. On the other hand, if we're just looking to
					// introduce some concavity, this does the job.
					float3 PlaneNormal = normalize( T /*GetTangentOutput0(MaterialParameters)*/);
					float3 CausticNormal = normalize( lerp( PlaneNormal, -GBufferWorldNormal, IrisMask*IrisDistance ) );
					float2 CausticNormalOct  = UnitVectorToOctahedron( CausticNormal );
					float2 CausticNormalDelta = ( CausticNormalOct - WorldNormalOct ) * 0.5 + (128.0/255.0);
					Out.Metallic = CausticNormalDelta.x;
					Out.Specular = CausticNormalDelta.y;
				#else
					float3 PlaneNormal = GBufferWorldNormal;
					Out.Metallic = 128.0/255.0;
					Out.Specular = 128.0/255.0;
				#endif

				// IrisNormal CustomData.yz
				//#if NUM_MATERIAL_OUTPUTS_CLEARCOATBOTTOMNORMAL > 0
				//	float3 IrisNormal = normalize( ClearCoatBottomNormal0(MaterialParameters) );
				//	#if MATERIAL_TANGENTSPACENORMAL
				//		IrisNormal = normalize( TransformTangentVectorToWorld( MaterialParameters.TangentToWorld, IrisNormal ) );
				//	#endif
				//#else
				//	float3 IrisNormal = PlaneNormal;
				//#endif

				float2 IrisNormalOct  = UnitVectorToOctahedron( IrisNormal );
				float2 IrisNormalDelta = ( IrisNormalOct - WorldNormalOct ) * 0.5 + (128.0/255.0);
				Out.CustomData.yz = IrisNormalDelta;
			#else
				Out.Metallic = IrisDistance;

				#if NUM_MATERIAL_OUTPUTS_GETTANGENTOUTPUT > 0
					float3 Tangent = IrisNormal;
					Out.CustomData.yz = UnitVectorToOctahedron( normalize(Tangent) ) * 0.5 + 0.5;
				#endif
			#endif

			#if SHADING_PATH_MOBILE
				#if MATERIAL_SHADINGMODEL_EYE_USE_CURVATURE
					Out.Curvature = Metallic;
				#else
					Out.Curvature = CalculateCurvature(GBufferWorldNormal, WorldPosition_CamRelative);
				#endif

					Out.Curvature = clamp(Out.Curvature, 0.001f, 1.0f);
			#endif
			}
			else
#endif
#if MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (SSSType == SSS_TYPE_DIFFUSION_PROFILE)  // Thin and not thin, Subsurface and TwoSided Foliage
			{
				Out.SubsurfaceProfileRadiusScale = SLAB_SSSPROFILERADIUSSCALE(CurrentBSDF);
				Out.CustomData.rgb = EncodeSubsurfaceProfile(Out.SubsurfaceProfileIDFloat);
				Out.CustomData.a   = Out.SubsurfaceProfileRadiusScale;

				const float3 GBufferWorldNormal = N;
				const float Opacity = SLAB_SSSPROFILERADIUSSCALE(CurrentBSDF);
					
				// Optimization: if opacity is 0 then revert to default shading model 
			#if SUBSURFACE_PROFILE_OPACITY_THRESHOLD
				if (Opacity > SSSS_OPACITY_THRESHOLD_EPS)
			#endif
				{
				#if SHADING_PATH_MOBILE
					#if MATERIAL_SUBSURFACE_PROFILE_USE_CURVATURE
					Out.Curvature = MobileShadingPathCurvature;
					#else
					Out.Curvature = CalculateCurvature(GBufferWorldNormal, WorldPosition_CamRelative);
					#endif

					Out.Curvature = clamp(Out.Curvature, 0.001f, 1.0f);
				#endif
				}
			#if SUBSURFACE_PROFILE_OPACITY_THRESHOLD
				else
				{
					Out.ShadingModelID = SHADINGMODELID_DEFAULT_LIT;
					Out.CustomData = 0;
				}
			#endif
			}
			else
#endif
#if MATERIAL_SHADINGMODEL_SUBSURFACE || MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP)
			{
				Out.CustomData.rgb = EncodeSubsurfaceColor(Out.SubsurfaceColor);
				Out.CustomData.a   = SubstrateSubSurfaceGetWrapOpacityFromAnisotropy(SLAB_SSSPHASEANISOTROPY(CurrentBSDF));
			}
			else
#endif
#if MATERIAL_SHADINGMODEL_CLOTH || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (BSDF_GETHASFUZZ(CurrentBSDF))
			{
				Out.CustomData.rgb = EncodeSubsurfaceColor(SLAB_FUZZ_COLOR(CurrentBSDF));
				Out.CustomData.a   = SLAB_FUZZ_AMOUNT(CurrentBSDF); 
				Out.IndirectIrradianceScale *= 1 - Out.CustomData.a;
			}
			else
#endif
#if MATERIAL_SHADINGMODEL_CLEAR_COAT || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (BSDF_GETHASHAZINESS(CurrentBSDF))
			{
				FHaziness LocalHaziness = UnpackHaziness(SLAB_HAZINESS(CurrentBSDF));
				if (LocalHaziness.bSimpleClearCoat)
				{
					Out.CustomData.x = LocalHaziness.Weight;
					Out.CustomData.y = LocalHaziness.Roughness;

				#if CLEAR_COAT_BOTTOM_NORMAL
					Out.CustomData.wz = LocalHaziness.BottomNormalOct.xy;
				#else
					Out.CustomData.wz = DEFAULT_CLEAR_COAT_BOTTOM_NORMAL_OCT;
				#endif
				}
			}
			else
#endif
#if MATERIAL_SHADINGMODEL_TOON || !MATERIAL_IS_SUBSTRATEHIDDENCONVERSION
				if (BSDFType == SUBSTRATE_BSDF_TYPE_TOON)
			{
				Out.ShadingModelID = SHADINGMODELID_SUBSTRATE_TOON;

				// Data stored using common Getxxx functions above:
				//	- BaseColor
				//	- Roughness
				//	- Metallic
				//	- Specular
				//  - Anisotropy is stored in the anisotropy buffer

				uint PackedToonCustomData = PackToonCustomData(TOON_PATTERNUVS(CurrentBSDF), TOON_PROFILEID(CurrentBSDF), Dither);
				Out.CustomData = UnpackRGBA8(PackedToonCustomData);
			}
			else
#endif
			// In case of an else, we need to provide a code block.
			{}
		}
		#undef CurrentBSDF
	}

	if (!LegacyShadingModelSupportsAnisotropy(Out.ShadingModelID))
	{
		// Legacy Subsurface profile did not support anisotropy so enforcing that to 0 here for backward compatibility.
		Out.Anisotropy = 0;
	}

	return Out;
}

#endif // SUBSTRATE_ENABLED, SUBSTRATE_MATERIAL_EXPORT_TYPE, etc.

