// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

///////////////////////////////////////////////////////////////////////////////
// Material buffer writing

#ifndef SUBSTRATE_VRS_WRITE
#define SUBSTRATE_VRS_WRITE 0
#endif

// * InQuadPixelWriteMask: mask describing which pixels needs to be written
// * InQuadPixelPos      : pixel coord of top left quad pixel. All mask's offset are expressed relative to this quad pixel

void WriteDataToBuffer(RWTexture2DArray<uint> OutBuffer, uint InData, uint3 InCoord, uint InQuadPixelWriteMask, uint2 InQuadPixelPos)
{
	if (InQuadPixelWriteMask > 0u)
	{
	#if SUBSTRATE_VRS_WRITE
		BRANCH
		if (InQuadPixelWriteMask & 1u)
	#endif
		{
			OutBuffer[uint3(InQuadPixelPos, InCoord.z)] = InData;
		}
	#if SUBSTRATE_VRS_WRITE
		BRANCH
		if (InQuadPixelWriteMask & 2u) // Copy H
		{
			OutBuffer[uint3(InQuadPixelPos.xy + uint2(1, 0), InCoord.z)] = InData;
		}
		BRANCH
		if (InQuadPixelWriteMask & 4u) // Copy V
		{
			OutBuffer[uint3(InQuadPixelPos.xy + uint2(0, 1), InCoord.z)] = InData;
		}
		BRANCH
		if (InQuadPixelWriteMask & 8u) // Copy D
		{
			OutBuffer[uint3(InQuadPixelPos.xy + uint2(1, 1), InCoord.z)] = InData;
		}
	#endif
	}
}

void WriteDataToBuffer(RWTexture2D<float3> OutBuffer, float3 InData, uint2 InCoord, uint InQuadPixelWriteMask, uint2 InQuadPixelPos)
{
	if (InQuadPixelWriteMask > 0u)
	{
	#if SUBSTRATE_VRS_WRITE
		BRANCH
		if (InQuadPixelWriteMask & 1u)
	#endif
		{
			OutBuffer[InQuadPixelPos] = InData;
		}
	#if SUBSTRATE_VRS_WRITE
		BRANCH
		if (InQuadPixelWriteMask & 2u) /* Copy H */
		{
			OutBuffer[InQuadPixelPos + uint2(1, 0)] = InData;
		}
		BRANCH
		if (InQuadPixelWriteMask & 4u) /* Copy V */
		{
			OutBuffer[InQuadPixelPos + uint2(0, 1)] = InData;
		}
		BRANCH
		if (InQuadPixelWriteMask & 8u) /* Copy D */
		{
			OutBuffer[InQuadPixelPos + uint2(1, 1)] = InData;
		}
	#endif
	}
}