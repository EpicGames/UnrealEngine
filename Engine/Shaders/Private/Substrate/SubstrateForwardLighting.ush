// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define USE_SUBSTRATE_FORWARD_LIGHTING_COMMON 1
#include "SubstrateLightingCommon.ush"
#include "../LightData.ush"

#if VIRTUAL_SHADOW_MAP
#include "../BlueNoise.ush"
#include "../VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#if SUPPORT_VSM_FOWARD_QUALITY > 0
#include "../VirtualShadowMaps/VirtualShadowMapProjectionFilter.ush"
#include "../VirtualShadowMaps/VirtualShadowMapProjectionDirectional.ush"
#endif // SUPPORT_VSM_FOWARD_QUALITY
#endif // VIRTUAL_SHADOW_MAP

#define FORWARD_PER_PIXEL_SHADING (FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || MATERIAL_SHADINGMODEL_SINGLELAYERWATER)

// Forward declarations
void GetVolumeLightingNonDirectional(float4 AmbientLightingVector, float3 DiffuseColor, inout float3 InterpolatedLighting, out float4 VolumeLighting);
void GetVolumeLightingDirectional(float4 AmbientLightingVector, float3 DirectionalLightingVector, float3 WorldNormal, float3 DiffuseColor, float DirectionalLightingIntensity, inout float3 InterpolatedLighting, out float4 VolumeLighting);

float3 GetWorldBentNormalZero(in FMaterialPixelParameters MaterialParameters);

FShadingOcclusion ApplyBentNormal(
	in half3 CameraVector,
	in half3 WorldNormal,
	in float3 WorldBentNormal0,
	in half Roughness,
	in half MaterialAO);

void GetPrecomputedIndirectLightingAndSkyLight(
	FMaterialPixelParameters MaterialParameters,
	FVertexFactoryInterpolantsVSToPS Interpolants,
	FBasePassInterpolantsVSToPS BasePassInterpolants,
	VTPageTableResult LightmapVTPageTableResult,
	bool bEvaluateBackface,
	float3 DiffuseDir,
	float3 VolumetricLightmapBrickTextureUVs,
	out float3 OutDiffuseLighting,
	out float3 OutSubsurfaceLighting,
	out float OutIndirectIrradiance);

// This code should map to GetForwardDirectLightingSplit
float3 SubstrateForwardLighting(
	uint EyeIndex,
	float4 SvPosition,
	FSubstrateIntegrationSettings Settings,
	FBasePassInterpolantsVSToPS BasePassInterpolants,
	FVertexFactoryInterpolantsVSToPS Interpolants,
	VTPageTableResult LightmapVTPageTableResult,
	float3 VolumetricLightmapBrickTextureUVs,
	FMaterialPixelParameters MaterialParameters,
	float SceneDepth,
	float2 ScreenUV,
	FSubstratePixelHeader SubstratePixelHeader,
	FSubstrateData SubstrateData,
	inout float3 OutTransmittancePreCoverage,
	inout float OutCoverage
	)
{
	const uint PrimitiveId = MaterialParameters.PrimitiveId;
	const uint2 PixelPos = uint2(SvPosition.xy);
	const float Dither = InterleavedGradientNoise(PixelPos, View.StateFrameIndexMod8);
	const float3 AbsoluteWorldPosition = WSDemote(GetWorldPosition(MaterialParameters));
	const float3 TranslatedWorldPosition = MaterialParameters.WorldPosition_CamRelative;
	const float3 LightingPositionOffset = MaterialParameters.LightingPositionOffset;
	float3 V = MaterialParameters.CameraVector;

	float3 Color = 0;
	OutCoverage = 0.0f;
	OutTransmittancePreCoverage = 1.0f;

	float4 DynamicShadowFactors = 1;
#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	DynamicShadowFactors = GetForwardDynamicShadowFactors(ScreenUV); // does that match between sub0/1?
#endif

	float SpecularScale = 1;
#if TRANSLUCENCY_ANY_VOLUMETRIC
	// No specular on volumetric translucency lighting modes
	SpecularScale = 0;
#endif

	uint PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask(PrimitiveId);

#if FORWARD_PER_PIXEL_SHADING
	// Create the common directional light data use for all layers
	const FDirectionalLightData DirectionalLightData = GetDirectionalLightData();
	
	float4 PreviewShadowMapChannelMask = 1;
	uint DirLightingChannelMask = LIGHTING_CHANNEL_MASK;
	FDeferredLightData DirLightData = ConvertToDeferredLight(DirectionalLightData, SpecularScale, PreviewShadowMapChannelMask, DirLightingChannelMask);

	// We want to force the directional light shadow when using water material to see shadow on the water. This could be an option later.
#if DISABLE_FORWARD_DIRECTIONAL_LIGHT_SHADOW
	float4 DirLightAttenuation = float4(1, 1, 1, 1);
#elif ((MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED) && !MATERIAL_SHADINGMODEL_SINGLELAYERWATER)
	float DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);

	// In the forward shading path we can't separate per-object shadows from CSM, since we only spend one light attenuation channel per light
	// If CSM is enabled (distance fading to precomputed shadowing is active), treat all of our dynamic shadowing as whole scene shadows that will be faded out at the max CSM distance
	// If CSM is not enabled, allow our dynamic shadowing to coexist with precomputed shadowing
	float PerObjectShadowing = DirLightData.DistanceFadeMAD.y < 0.0f ? 1.0f : DynamicShadowing;
	float WholeSceneShadowing = DirLightData.DistanceFadeMAD.y < 0.0f ? DynamicShadowing : 1.0f;
		
	float4 DirLightAttenuation = float4(WholeSceneShadowing.xx, PerObjectShadowing.xx);
#else
	DirLightData.ShadowedBits = 1;
	DirLightData.ShadowMapChannelMask.x = 1;
	bool bUnused = false;
	float DirLightDynamicShadowFactor = ComputeDirectionalLightDynamicShadowing(TranslatedWorldPosition, SceneDepth, bUnused);
			
#if VIRTUAL_SHADOW_MAP
	BRANCH
	if ( ForwardLightStruct.DirectionalLightVSM != INDEX_NONE )
	{
		#if SUPPORT_VSM_FOWARD_QUALITY == 1
		{
			const FLightShaderParameters DirectionalLight = ConvertToLightShaderParameters(DirLightData);

			const float ScreenRayLengthWorld = VirtualShadowMap.ScreenRayLength * View.ClipToView[1][1] * SceneDepth;
			float SMRTRayOffset = ScreenRayLengthWorld;

			// Use the surface normal instead of the layer's normal, for a single evaluation before runing throught the SubstrateTree
			const half3 WorldNormal = normalize(MaterialParameters.TangentToWorld[2]);

			FVirtualShadowMapSampleResult VirtualShadowMapSample = TraceDirectional(
				ForwardLightStruct.DirectionalLightVSM,
				DirectionalLight,
				PixelPos,
				SceneDepth,
				TranslatedWorldPosition,
				SMRTRayOffset,
				Dither,
				WorldNormal);

			FilterVirtualShadowMapSampleResult(PixelPos, VirtualShadowMapSample);

			DirLightDynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
		}
		#else
		{
			FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapDirectional(ForwardLightStruct.DirectionalLightVSM, TranslatedWorldPosition);
			DirLightDynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
		}
		#endif
	}
#endif

	float4 DirLightAttenuation = float4(DirLightDynamicShadowFactor.x, DirLightDynamicShadowFactor.x, 1, 1);
#endif
#endif // FORWARD_PER_PIXEL_SHADING

	uint GridIndex = 0;
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	GridIndex = ComputeLightGridCellIndex((uint2)((MaterialParameters.SvPosition.xy - ResolvedView.ViewRectMin.xy) * View.LightProbeSizeRatioAndInvSizeRatio.zw), MaterialParameters.SvPosition.w, EyeIndex);
#endif



	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		// Write Irradiance/AO data into header in order to be retrieve correctly during lighting
		#if SUBSTRATE_INLINE_SHADING
		HEADER_SETIRRADIANCE_AO(SubstratePixelHeader.State, SubstratePackIrradianceAndOcclusion(SubstratePixelHeader.IrradianceAO, 0));
		#endif

		// Update tree (coverage/transmittance/luminace weights)
		SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V, Settings, OutCoverage, OutTransmittancePreCoverage);

		//
		// Forward lighting
		//
		Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
		{
		#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

			if (SubstrateIsBSDFVisible(CurrentBSDF))
			{
				FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;	// Fake unused in SubstrateCreateBSDFContext when using Forward inline shading
				// Starting with environment lighting, the context does not specify a light
				FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, V);
					
				////
				//// Evaluate environment lighting
				////

				// Compute the bent normal from the BSDF normal
				const float Roughness = SubstrateGetBSDFRoughness(CurrentBSDF);
				const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, SubstrateBSDFContext.N, GetWorldBentNormalZero(MaterialParameters), Roughness,  SubstrateGetAO(SubstratePixelHeader));

				float IndirectOcclusion = 1.0f;
				float IndirectIrradiance = 0.0f; 

				// 1. Diffuse Sky evaluation using bent normal
				{
					// Update a context specific for environment lighting
					FSubstrateBSDFContext EnvBSDFContext = SubstrateBSDFContext;
					EnvBSDFContext.N = ShadingOcclusion.BentNormal;
					EnvBSDFContext.SubstrateUpdateBSDFContext(EnvBSDFContext.L);

					// And evaluate diffuse parameters
					FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(EnvBSDFContext, false/*bEnableSpecular*/, Settings);
					float3 DiffuseNormal = SubstrateEnvLight.DiffuseNormal;
					float3 DiffuseColorForIndirect = SubstrateEnvLight.DiffuseWeight;
					float3 SubsurfaceColorForIndirect = SubstrateEnvLight.DiffuseBackFaceWeight;

					// Evaluate diffuse lighting
					{
						const bool bEvaluateBackface = any(SubsurfaceColorForIndirect > 0.0);

						float3 DiffuseIndirectLighting = 0.0f;
						float3 SubsurfaceIndirectLighting = 0.0f;

					#if (MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE) && PROJECT_SUPPORTS_LUMEN && !FORWARD_SHADING
						if (IsLumenTranslucencyGIEnabled())
						{
							if (any((DiffuseColorForIndirect + SubsurfaceColorForIndirect) > 0.0f)) // Lumen evaluation can be skipped if all diffuse components are 0.
							{
								// Lumen Dynamic GI + shadowed Skylight
								FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(MaterialParameters.AbsoluteWorldPosition, ResolvedView.WorldToClip, true);

							#if TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

								FOneBandSHVectorRGB TranslucencyGISH1;
								TranslucencyGISH1.R.V = TranslucencyGISH.R.V.x;
								TranslucencyGISH1.G.V = TranslucencyGISH.G.V.x;
								TranslucencyGISH1.B.V = TranslucencyGISH.B.V.x;

								FOneBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH1(1);
								DiffuseIndirectLighting += max(float3(0,0,0), DotSH1(TranslucencyGISH1, DiffuseTransferSH)) / PI;

							#else
								// Diffuse convolution
								FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(DiffuseNormal, 1);
								DiffuseIndirectLighting += max(half3(0,0,0), DotSH(TranslucencyGISH, DiffuseTransferSH)) / PI;

								#if SHADINGMODEL_REQUIRES_BACKFACE_LIGHTING
								if (bEvaluateBackface)
								{
									FTwoBandSHVector SubsurfaceTransferSH = CalcDiffuseTransferSH(-DiffuseNormal, 1);
									SubsurfaceIndirectLighting += max(half3(0,0,0), DotSH(TranslucencyGISH, SubsurfaceTransferSH)) / PI;
								}
								#endif
							#endif
							}
						}
						else
					#endif
						if (UseBasePassSkylight > 0) // Non Lumen evaluation cannot be skipped if all diffuse components are 0 due to IndirectIrradiance which might still contribute via GetImageBasedReflectionSpecular
						{
							GetPrecomputedIndirectLightingAndSkyLight(
								MaterialParameters,
								Interpolants,
								BasePassInterpolants,
								LightmapVTPageTableResult,
								bEvaluateBackface,
								DiffuseNormal,
								VolumetricLightmapBrickTextureUVs,
								DiffuseIndirectLighting,
								SubsurfaceIndirectLighting,
								IndirectIrradiance);
						}

					#if FORWARD_SHADING && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED)
						float2 NearestResolvedDepthScreenUV = CalculateNearestResolvedDepthScreenUV(ScreenUV, MaterialParameters.ScreenPosition.w);

						IndirectOcclusion = GetIndirectOcclusion(NearestResolvedDepthScreenUV, SubstratePixelHeader.HasDynamicIndirectShadowCasterRepresentation());
						DiffuseIndirectLighting *= IndirectOcclusion;
						SubsurfaceIndirectLighting *= IndirectOcclusion;
						IndirectIrradiance *= IndirectOcclusion;
					#endif

						// For diffuse, we specify a perpendicular to the surface light direction for the transmittance to light to not be view dependent.
						const float DiffuseEnvLightingNoL = 1.0f;

						Color += (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColorForIndirect) * 
							LuminanceWeight(DiffuseEnvLightingNoL, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.DiffuseColor, ShadingOcclusion.DiffOcclusion);
					}
				}

				// 2. Specular evaluation (using regular normal)
				#if FORWARD_PER_PIXEL_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME
				// Every components of forward shading reflection (Sky, SSR, Lumen) are enabled/disabled within GetImageBasedReflectionSpecular.
				{
					// Evaluate specular parameters without bent normal
					FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true/*bEnableSpecular*/, Settings);

					// Evaluate specular lighting for the main lobe
					if (any(SubstrateEnvLight.SpecularWeight > 0.0f))
					{
						int SingleCaptureIndex = GetPrimitiveData(MaterialParameters).SingleCaptureIndex;
						Color += GetImageBasedReflectionSpecular(MaterialParameters, SubstrateEnvLight.SpecularDirection, SubstrateEnvLight.SpecularSafeRoughness, IndirectIrradiance, GridIndex, SingleCaptureIndex) *
								SubstrateEnvLight.SpecularWeight * IndirectOcclusion * LuminanceWeight(SubstrateBSDFContext, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.SpecularColor, ShadingOcclusion.SpecOcclusion);
					}
					// And for the second lobe if needed
					if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
					{
						int SingleCaptureIndex = GetPrimitiveData(MaterialParameters).SingleCaptureIndex;
						Color += GetImageBasedReflectionSpecular(MaterialParameters, SubstrateEnvLight.SpecularHazeDirection, SubstrateEnvLight.SpecularHazeSafeRoughness, IndirectIrradiance, GridIndex, SingleCaptureIndex) *
							SubstrateEnvLight.SpecularHazeWeight * IndirectOcclusion * LuminanceWeight(SubstrateBSDFContext, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.SpecularColor, ShadingOcclusion.SpecOcclusion);
					}
				}
				#endif

				////
				//// Evaluate translucent lighting volume / vertex lighting.
				////

				// In deferred shading, the translucenyt volume is used by 
				//	- TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL
				//	- TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL
				//	- TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL
				//	- TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL
				//	- TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME
				// In forward, things change (for some reasons...)
				//  - TRANSLUCENCY_PERVERTEX_FORWARD_SHADING make sure TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL and TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL works the same way.
				//  - TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL will follow the SubstrateForwardLightingCommon path and directionality is removed using NON_DIRECTIONAL_DIRECT_LIGHTING
				//  - TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL will be same as TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING but without the environment specular contribution
				//	- TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME becomes the same as TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING
				// SUBSTRATE_TODO simplify all that and have forward map to the same technique as in deferred.

			#if TRANSLUCENCY_PERVERTEX_FORWARD_SHADING || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || (!FORWARD_SHADING && (TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME))
				float4 VolumeLighting;
				float3 SurfaceLighting = 0;
				float3 InnerVolumeUVs;
				float3 OuterVolumeUVs;
				float FinalLerpFactor;
				ComputeVolumeUVs(TranslatedWorldPosition, LightingPositionOffset, InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);

				FSubstrateEvaluateResult BSDFEvaluate = SubstrateEvaluateBSDF(SubstrateBSDFContext, Settings);

			#if TRANSLUCENCY_PERVERTEX_FORWARD_SHADING

				SurfaceLighting = BasePassInterpolants.VertexDiffuseLighting * BSDFEvaluate.DiffuseColor;

			#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL

				// ForwardShadingNote: when forward shading is enabled, this will follow the special TRANSLUCENCY_PERVERTEX_FORWARD_SHADING path above
				GetVolumeLightingNonDirectional(float4(BasePassInterpolants.AmbientLightingVector, 1), BSDFEvaluate.DiffuseColor, SurfaceLighting, VolumeLighting);

			#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL

				// ForwardShadingNote: when forward shading is enabled, this will follow the special TRANSLUCENCY_PERVERTEX_FORWARD_SHADING path above
				GetVolumeLightingDirectional(float4(BasePassInterpolants.AmbientLightingVector, 1), BasePassInterpolants.DirectionalLightingVector, SubstrateBSDFContext.N, BSDFEvaluate.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), SurfaceLighting, VolumeLighting);

			#elif TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL && !FORWARD_SHADING

				//ForwardShadingNote: when forward shading is enabled, SubstrateForwardLightingCommon path will be used 
				float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
				GetVolumeLightingNonDirectional(AmbientLightingVector, BSDFEvaluate.DiffuseColor, SurfaceLighting, VolumeLighting);

			#elif (TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME) && !FORWARD_SHADING

				//ForwardShadingNote: when forward shading is enabled, SubstrateForwardLightingCommon path will be used 
				float4 AmbientLightingVector = GetAmbientLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
				float3 DirectionalLightingVector = GetDirectionalLightingVectorFromTranslucentLightingVolume(InnerVolumeUVs, OuterVolumeUVs, FinalLerpFactor);
				GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, SubstrateBSDFContext.N, BSDFEvaluate.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), SurfaceLighting, VolumeLighting);

			#endif // TRANSLUCENCY_

				// SUBSTRATE_TODO: we should simply remove this self shadow code path TRANSLUCENT_SELF_SHADOWING and only use FOM to shadow the ? Translucent shadow should .
			#if (TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME) && TRANSLUCENT_SELF_SHADOWING
				// Only apply self shadowing if the shadow hasn't faded out completely
				if (TranslucentSelfShadow.DirectionalLightColor.a > 0)
				{
					// Determine the shadow space position
					// Apply a stable offset to the world position used for shadowing, which blurs out high frequency details in the shadowmap with many layers
					float4 HomogeneousShadowPosition = mul(float4(AbsoluteWorldPosition + LightingPositionOffset * View.TranslucencyLightingVolumeInvSize[0].w, 1), TranslucentSelfShadow.WorldToShadowMatrix);
					float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;
					float ShadowZ = 1 - HomogeneousShadowPosition.z;
					// Lookup the shadow density at the point being shaded
					float3 ShadowDensity = CalculateTranslucencyShadowingDensity(ShadowUVs, ShadowZ) / GetMaterialTranslucentMultipleScatteringExtinction();
					// Compute colored transmission based on the density that the light ray passed through
					float3 SelfShadowing = saturate(exp(-ShadowDensity * GetMaterialTranslucentSelfShadowDensityScale()));
					// Compute a second shadow gradient to add interesting information in the shadowed area of the first
					// This is a stop gap for not having self shadowing from other light sources
					float3 SelfShadowing2 = lerp(float3(1, 1, 1), saturate(exp(-ShadowDensity * GetMaterialTranslucentSelfShadowSecondDensityScale())), GetMaterialTranslucentSelfShadowSecondOpacity());
					SelfShadowing = SelfShadowing * SelfShadowing2;

					// Force unshadowed if we read outside the valid area of the shadowmap atlas
					// This can happen if the particle system's bounds don't match its visible area
					FLATTEN
					if (any(ShadowUVs < TranslucentSelfShadow.ShadowUVMinMax.xy || ShadowUVs > TranslucentSelfShadow.ShadowUVMinMax.zw))
					{
						SelfShadowing = 1;
					}

					// The volume lighting already contains the contribution of the directional light, 
					// So calculate the amount of light to remove from the volume lighting in order to apply per-pixel self shadowing
					// VolumeLighting.a stores all attenuation and opaque shadow factors
					float3 SelfShadowingCorrection = TranslucentSelfShadow.DirectionalLightColor.rgb * VolumeLighting.a * (1 - SelfShadowing);

					// Combine backscattering and directional light self shadowing
					SurfaceLighting = (BSDFEvaluate.DiffuseColor * max(VolumeLighting.rgb - SelfShadowingCorrection, 0));
				}
			#endif

				const float LightVolumeNoL = 1.0f;
				Color += SurfaceLighting * LuminanceWeight(LightVolumeNoL, CurrentBSDF) + BSDFEvaluate.EmissivePathValue * CurrentBSDF.LuminanceWeightV;

			#elif FORWARD_PER_PIXEL_SHADING


				////
				//// Evaluate emissive
				////
				Color += BSDF_GETEMISSIVE(CurrentBSDF) * CurrentBSDF.LuminanceWeightV;

				////
				//// Evaluate the single directional light selected for forward lighting.
				////
				BRANCH
				if (DirectionalLightData.HasDirectionalLight
			#if SUBSTRATE_TRANSLUCENT_MATERIAL
					&& DirectionalLightData.bAffectsTranslucentLighting > 0
			#endif
						)
				{
					float3 DirLightL = DirLightData.Direction;	// Already normalized
					float3 ToDirLight = DirLightL;
					float DirLightMask = 1;
					if (DirLightData.bRadialLight)
					{
						DirLightMask = GetLocalLightAttenuation(TranslatedWorldPosition, DirLightData, ToDirLight, DirLightL);
					}

					float DirectionalLightCloudShadow = 1.0f;
				#if NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
					DirectionalLightCloudShadow = BasePassInterpolants.VertexCloudShadow;
				#endif

					SubstrateBSDFContext.SubstrateUpdateBSDFContext(ToDirLight);

					const bool bIsDirLight = true;
					// Get base precompute shadow factors.
					float4 PrecomputedShadowFactor = GetPrecomputedShadowFactors(SubstratePixelHeader, TranslatedWorldPosition, bIsDirLight);
					// Mix in the static light map shadow factors.
					PrecomputedShadowFactor *= GetPrecomputedShadowMasks(LightmapVTPageTableResult, Interpolants, MaterialParameters, VolumetricLightmapBrickTextureUVs);

					FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
					Color += SubstrateForwardLightingCommon(
						Dither,
						Settings,
						DirLightData,
						ToDirLight,
						DirLightMask,
						DirLightAttenuation,
						InitRectTexture(),
						DirLightingChannelMask,
						PrimitiveLightingChannelMask,
						PrecomputedShadowFactor,
						TranslatedWorldPosition,
						SceneDepth,
						LuminanceWeight(SubstrateBSDFContext, CurrentBSDF),
						SubstratePixelHeader,
						SubstrateBSDFContext,
						BSDFEvaluate) * DirectionalLightCloudShadow;
				}

				////
				//// Evaluate local lights
				////

			#if !DISABLE_FORWARD_LOCAL_LIGHTS && (FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING) // Forbidden for Single Layer Water
				// Basic implementation for FORWARD_LOCAL_LIGHTS where local lights are reloaded per BSDF layer. It is fast by default when layer==1.
				const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

				// Limit max to ForwardLightStruct.NumLocalLights.
				// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
				const uint NumLightsInGridCell = min(CulledLightsGridHeader.NumLights, GetMaxLightsPerCell());

				const bool bIsDirLight = false;
				// Get base precompute shadow factors.
				float4 PrecomputedShadowFactor = GetPrecomputedShadowFactors(SubstratePixelHeader, TranslatedWorldPosition, bIsDirLight);
				// Mix in the static light map shadow factors.
				PrecomputedShadowFactor *= GetPrecomputedShadowMasks(LightmapVTPageTableResult, Interpolants, MaterialParameters, VolumetricLightmapBrickTextureUVs);

				LOOP
				for (uint GridLightListIndex = 0; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
				{
					const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);
		
				#if MATERIALBLENDING_ANY_TRANSLUCENT
					if(UnpackAffectsTranslucentLighting(LocalLight) == 0)
					{
						continue;
					}
				#endif

					half4 PreviewShadowMapChannelMask = 1;
					uint  LocalLightChannelMask = LIGHTING_CHANNEL_MASK;
					FDeferredLightData LightData = ConvertToDeferredLight(LocalLight, 1.0f /*SpecularScale*/, PreviewShadowMapChannelMask, LocalLightChannelMask);

					#if USE_IES_PROFILE
					LightData.Color *= ComputeLightProfileMultiplier(TranslatedWorldPosition, LightData.TranslatedWorldPosition, -LightData.Direction, LightData.Tangent, LightData.IESAtlasIndex);
					#endif

					// Rect light optionally supported in forward.
					// * This is not done explicitely in the legacy path (resulting in no lighting) but we do it for Substrate in order to avoid visual artefact.
					// * This can be removed when LocalLight properly supports rect lights.
					if (LightData.bRectLight && !SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT)
					{
						continue;
					}

					// When disabled, force the compiler to remove the texture atlas sampling to avoid taking an extra texture sampler
					#if !SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT
					LightData.bRectLight = false;
					#endif
					const FRectTexture LocalRectTexture = ConvertToRectTexture(LightData);

					float DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);
					float4 LightAttenuation = float4(1, 1, DynamicShadowing.x, DynamicShadowing.x);

					float3 L = LightData.Direction;	// Already normalized
					float3 ToLight = L;
					float LightMask = 1;
					if (LightData.bRadialLight)
					{
						LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
					}

					#if VIRTUAL_SHADOW_MAP && SUPPORT_SHADOWED_LOCAL_LIGHT_ON_FORWARD_LIT_TRANSLUCENT
					BRANCH
					if (LocalLight.Internal.VirtualShadowMapId != INDEX_NONE)
					{
						float DynamicShadowFactor = 1.f;
						LightData.ShadowedBits = 1;
						#if SUPPORT_VSM_FOWARD_QUALITY == 1
						{
							const FLightShaderParameters LocalLightShaderParameters = ConvertToLightShaderParameters(LightData);

							const float ScreenRayLengthWorld = VirtualShadowMap.ScreenRayLength * View.ClipToView[1][1] * Depth;
							float SMRTRayOffset = ScreenRayLengthWorld;

							// Use the surface normal instead of the layer's normal, for a single evaluation before runing throught the SubstrateTree
							const half3 WorldNormal = normalize(MaterialParameters.TangentToWorld[2]);

							FVirtualShadowMapSampleResult VirtualShadowMapSample = TraceLocalLight(
								LocalLight.VirtualShadowMapId,
								LocalLightShaderParameters,
								PixelPos,
								SceneDepth,
								TranslatedWorldPosition,
								SMRTRayOffset,
								Dither,
								WorldNormal);

							FilterVirtualShadowMapSampleResult(PixelPos, VirtualShadowMapSample);

							DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
						}
						#else
						{
							FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapLocal(LocalLight.Internal.VirtualShadowMapId, TranslatedWorldPosition);
							DynamicShadowFactor *= VirtualShadowMapSample.ShadowFactor;
						}
						#endif

						LightAttenuation = DynamicShadowFactor;
					}
					#endif

					// Update the Substrate BSDF context accordgin to the new L
					SubstrateBSDFContext.SubstrateUpdateBSDFContext(ToLight);

					FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
					Color += SubstrateForwardLightingCommon(
						Dither,
						Settings,
						LightData,
						ToLight,
						LightMask,
						LightAttenuation,
						LocalRectTexture,
						LocalLightChannelMask,
						PrimitiveLightingChannelMask,
						PrecomputedShadowFactor,
						TranslatedWorldPosition,
						SceneDepth,
						LuminanceWeight(SubstrateBSDFContext, CurrentBSDF),
						SubstratePixelHeader,
						SubstrateBSDFContext,
						BSDFEvaluate);
				}

			#endif // !DISABLE_FORWARD_LOCAL_LIGHTS

			#endif // Lighting technique


			}// SubstrateIsBSDFVisible
		#undef CurrentBSDF
		}
	}


	return Color;
}


