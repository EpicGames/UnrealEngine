// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"

// Optimization: if opacity is 0 then revert to default shading model 
// This is for matching legacy behavior. See ShadingModelsMaterial.ush.
#define SUBSURFACE_PROFILE_OPACITY_THRESHOLD 1

#define SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY (SUBSTRATE_TRANSLUCENT_MATERIAL || SUBSTRATE_MATERIAL_EXPORT_TYPE == SUBSTRATE_MATERIAL_EXPORT_OPACITY)

///////////////////////////////////////////////////////////////////////////////////////////////////
// Helpers

void GetEyeNormals(
	float IrisMask,
	float IrisDistance,
	in float3 InNormal,
	in float3 InClearCoatNormal,
	in float3 InCustomTangent,
	inout float3 OutIrisNormal,
	inout float3 OutIrisPlaneNormal);

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FSubstrateLegacyParameters
{
	FSubstratePixelFootprint PixelFootprint;
	bool UseMetalness;
	float3 DiffuseAlbedo;
	float3 F0;
	float3 F90;

	float3 RawNormal;
	float3 BaseColor;
	float  Specular;
	float  Metallic;
	float  Roughness;
	float  Anisotropy;
	uint   SSSProfileID;
	bool bSupportDefaultSSSProfile;
	float3 SSSMFP;
	float  SSSMFPScale;
	float  SSSPhaseAniso;
	uint   SSSType;
	float3 Emissive;
	float  SecondRoughness;
	float  SecondRoughnessWeight;
	bool   SecondRoughnessAsSimpleClearCoat;
	bool   ClearCoatUseBottomNormal;
	float3 ClearCoatBottomNormal;
	float  FuzzAmount;
	float3 FuzzColor;
	float  Thickness;
	bool   bIsThinSurface;
	uint   SharedLocalBasisIndex;

	float Weight;
};

FSubstrateLegacyParameters InitSubstrateLegacyParameters(FSubstratePixelFootprint InPixelFootprint, uint InSharedLocalBasisIndex, float InWeight)
{
	const float3 Zeros = float3(0, 0, 0);

	FSubstrateLegacyParameters Out;

	Out.UseMetalness = true;
	Out.DiffuseAlbedo = Zeros;
	Out.F0 = Zeros;
	Out.F90 = Zeros;

	Out.RawNormal = Zeros;
	Out.BaseColor = Zeros;
	Out.Specular = 0.5f;
	Out.Metallic = 0.f;
	Out.Roughness = 0.5f;
	Out.Anisotropy = 0.f;
	Out.SSSProfileID = 0;
	Out.bSupportDefaultSSSProfile = true;
	Out.SSSMFP = Zeros;
	Out.SSSMFPScale = 1.f;
	Out.SSSPhaseAniso = 0.f;
	Out.SSSType = SSS_TYPE_NONE;
	Out.Emissive = Zeros;
	Out.SecondRoughness = 0.0f;
	Out.SecondRoughnessWeight = 0.0f;
	Out.SecondRoughnessAsSimpleClearCoat = false;
	Out.ClearCoatUseBottomNormal = false;
	Out.FuzzAmount = 0.f;
	Out.FuzzColor = Zeros;
	Out.Thickness = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
	Out.bIsThinSurface = false;
	Out.SharedLocalBasisIndex = InSharedLocalBasisIndex;
	Out.Weight = InWeight;
	Out.PixelFootprint = InPixelFootprint;
	return Out;
}

FSubstrateData CreateLegacySlab(
	FSubstrateLegacyParameters In, inout uint SharedLocalBasisTypes)
{
	const float3 Zeros = float3(0, 0, 0);
	const float3 Ones  = float3(1, 1, 1);

	float3 DiffuseAlbedo = In.DiffuseAlbedo;
	float3 F0 = In.F0;
	float3 F90 = In.F90;
	if (In.UseMetalness)
	{
		DiffuseAlbedo = ComputeDiffuseAlbedo(In.BaseColor, In.Metallic);
		F0 = ComputeF0(In.Specular, In.BaseColor, In.Metallic);
		F90 = Ones;
	}

	// Fixed layer structure for helping compiler to unroll and optimize shader
	return GetSubstrateSlabBSDF(
		In.PixelFootprint,
		In.RawNormal,										// Normal 
		DiffuseAlbedo,										// DiffuseAlbedo
		F0,													// F0
		F90,												// F90
		In.Roughness,										// Roughness
		In.Anisotropy,										// Anisotropy
		In.SSSProfileID,									// SSSProfileID
		In.bSupportDefaultSSSProfile,						// bSupportDefaultSSSProfile
		In.SSSMFP,											// SSSMFP
		In.SSSMFPScale,										// SSSMFPScale
		In.SSSPhaseAniso,									// SSSPhaseAniso
		In.SSSType,											// SSSType
		In.Emissive,										// Emissive
		In.SecondRoughness,									// SecondRoughness
		In.SecondRoughnessWeight,							// SecondRoughnessWeight
		In.SecondRoughnessAsSimpleClearCoat ? 1.0f : 0.0f,	// SecondRoughnessAsSimpleClearCoat
		In.ClearCoatUseBottomNormal ? 1.0f : 0.0f,			// ClearCoatUseSecondNormal;
		In.ClearCoatBottomNormal,							// ClearCoatBottomNormal;
		In.FuzzAmount,										// FuzzAmount
		In.FuzzColor,										// FuzzColor
		0.5 + 0.5 * In.Roughness,							// FuzzRoughness (adjust range because the legacy model has a very different response)
		1.0f, (GLINT_UV_TYPE)0,								// GlintData
		0.0f,												// Specular profile
		In.Thickness,
		In.bIsThinSurface,
		true,												// bIsAtTheBottomOfTopology, legacy always has a single BSDF being at the bottom and top at the same time.
		In.SharedLocalBasisIndex, SharedLocalBasisTypes);
}

// Convert legacy shading models - Dynamic
// This function can handle dynamic shading models (i.e., known only at runtime). 
// For this, the layer topology is 'fixed' and composed of two slabs vertically layered. This is done for 
// helping the compiler to unroll Substrate BSDF traversing and packing. In most cases, the bottom slab is 
// weighted by 0 and it will be removed once the data are written-out/packed.
FSubstrateData SubstrateConvertLegacyMaterialDynamic(
	FSubstratePixelFootprint PixelFootprint,
	float3 BaseColor, float Specular, float Metallic,
	float Roughness, float Anisotropy,
	float3 SubSurfaceColor, uint SubSurfaceProfileId,
	float ClearCoat, float ClearCoatRoughness,
	float3 Emissive,
	float Opacity,
	float3 ThinTranslucentTransmittanceColor,
	float ThinTranslucentSurfaceCoverage,
	float3 WaterScatteringCoefficients, float3 WaterAbsorptionCoefficients, float WaterPhaseG, float3 ColorScaleBehindWater,
	uint ShadingModel,
	float3 RawNormal,
	float3 RawTangent,
	float3 RawClearCoatNormal,
	float3 RawCustomTangent,
	uint SharedLocalBasisIndex,
	uint ClearCoatBottomNormal_SharedLocalBasisIndex,
	inout uint SharedLocalBasisTypes)
{
	const float DefaultThickness = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
	const float3 Zeros = float3(0, 0, 0);
	const float3 Ones  = float3(1, 1, 1);

	const float OneMinusOpacity = 1.f - Opacity;

	if (!LegacyShadingModelSupportsAnisotropy(ShadingModel))
	{
		// Legacy Subsurface profile did not support anisotropy so enforcing that to 0 here for backward compatibility.
		Anisotropy = 0;
	}

	// Can only mix Unlit / DefaultLit / Sub-Surface / Preintegrated-Skin / Subsurface-Profile / ClearCoat / Foliage / Cloth / Eye / Thin

	// Note: If hair or single layer water are enabled with other shading model, the code generation won't be ideal, as the compiler will struggle with unrolling the BSDFs
	#if MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	if (ShadingModel == SHADINGMODELID_SINGLELAYERWATER)
	{
		const float3 WaterExtinction = WaterScatteringCoefficients + WaterAbsorptionCoefficients;
		const float3 WaterAlbedo = WaterScatteringCoefficients / WaterExtinction;

		return GetSubstrateSingleLayerWaterBSDF(
			BaseColor, 				// BaseColor
			Metallic, 				// Metallic 
			Specular, 				// Specular
			Roughness, 				// Roughness 
			Emissive, 				// Emissive 
			Opacity,				// TopMaterialOpacity
			WaterAlbedo, 			// WaterAlbedo
			WaterExtinction, 		// WaterExtinction
			WaterPhaseG, 			// WaterPhaseG
			ColorScaleBehindWater,	// ColorScaleBehindWater
			SharedLocalBasisIndex);
	}
	#endif // MATERIAL_SHADINGMODEL_SINGLELAYERWATER
	#if MATERIAL_SHADINGMODEL_HAIR
	if (ShadingModel == SHADINGMODELID_HAIR)
	{
		return GetSubstrateHairBSDF(
			BaseColor,	   // BaseColor
			Metallic,	   // Scatter
			Specular,	   // Specular
			Roughness,	   // Roughness
			ClearCoat,	   // Backlit
			Emissive,	   // EmissiveColor
			false,		   // bComplexTransmittance
			SharedLocalBasisIndex);
	}
	#endif // MATERIAL_SHADINGMODEL_HAIR
	#if MATERIAL_SHADINGMODEL_EYE
	if (ShadingModel == SHADINGMODELID_EYE)
	{
		const float IrisMask = ClearCoat;
		const float IrisDistance = ClearCoatRoughness;

		float3 IrisNormal = RawNormal;
		float3 IrisPlaneNormal = RawNormal;
		GetEyeNormals(IrisMask, IrisDistance, RawNormal, RawClearCoatNormal, RawCustomTangent, IrisNormal, IrisPlaneNormal);

		return GetSubstrateEyeBSDF(
				BaseColor,	         // DiffuseAlbedo
				Roughness,	         // Roughness
				IrisMask,	         // IrisMask
				IrisDistance,        // IrisDistance
				IrisNormal,	         // IrisNormal
				IrisPlaneNormal,	 // IrisPlaneNormal
				SubSurfaceProfileId, // SSS profile
				Emissive,	         // EmissiveColor
				SharedLocalBasisIndex);
	}
	#endif // MATERIAL_SHADINGMODEL_EYE

	FSubstrateLegacyParameters LegacySlab = InitSubstrateLegacyParameters(PixelFootprint, SharedLocalBasisIndex, 1.0f);
	LegacySlab.RawNormal = RawNormal;

	float FinalWeight = 1.0f;
	#if MATERIAL_SHADINGMODEL_UNLIT
	if (ShadingModel == SHADINGMODELID_UNLIT)
	{
		// Unlit is handle with a emissive slab
		LegacySlab.BaseColor = Zeros;
		LegacySlab.Specular = 0.f;
		LegacySlab.Metallic = 0.f;
		LegacySlab.Roughness = 0.f;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.Emissive = Emissive;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_DEFAULT_LIT
	if (ShadingModel == SHADINGMODELID_DEFAULT_LIT)
	{
		LegacySlab.BaseColor = BaseColor;
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.Emissive = Emissive;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_SUBSURFACE
	if (ShadingModel == SHADINGMODELID_SUBSURFACE)
	{
		const float ThicknessInCm = SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM;
		const float3 MFPInCm = TransmittanceToMeanFreePath(SubSurfaceColor, ThicknessInCm * CENTIMETER_TO_METER) * METER_TO_CENTIMETER;

		LegacySlab.BaseColor = BaseColor;
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.SSSMFP = max(1e-05f, MFPInCm); // Ensure the MFP is not null to force the material as have SSS
		LegacySlab.SSSMFPScale = 1.0f;
		LegacySlab.SSSPhaseAniso = OneMinusOpacity; // Opaque-Thick: isotropic phase function, Thin: forward phase scattering function
		LegacySlab.Emissive = Emissive;
		LegacySlab.SSSType = SSS_TYPE_WRAP;
		LegacySlab.Thickness = ThicknessInCm;

		FinalWeight = 1.0f;

#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		// SUBSTRATE_TODO 
		// If we try to reduce the diffusion according to transmittance that will increase mfp and make the material disapear (make it optically thin).
		// Instead for now we use a mfp of 0 and make the material vanish according to opacity.
		// All this will be fixed when this material will be converted to LUT.
		const float Transmittance = OneMinusOpacity;
		LegacySlab.SSSMFP = 0;// MFPInCm;
		LegacySlab.SSSMFPScale = 1.0f;
		LegacySlab.Thickness = ThicknessInCm;

		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_PREINTEGRATED_SKIN
	if (ShadingModel == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		#if 0
		// This is a better conversion, but requires support for PerPixelSSS which is currently not support with Format=0 (we don't have GBuffer encoding for this yet)
		// Clamp MFP to MaxSSSDiffusionMFP, as the current SSS screen diffusion algorithm can be too noisy (current value is arbitrary)
		const float ThicknessInCm = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
		const float MaxSSSDiffusionMFP = 1.f;
		const float3 MFPInCm = min(TransmittanceToMeanFreePath(SubSurfaceColor, ThicknessInCm * CENTIMETER_TO_METER) * METER_TO_CENTIMETER, MaxSSSDiffusionMFP);
		const bool bSupportDefaultSSSProfile = false;
		#else
		// Use default profile MFP (e.g., human skin) as MFP value for converting hardcoded pre-integrated SSS texture for deferred material.
		const float3 MFPInCm = float3(1.0f, 0.088964f, 0.072095f) * 2.6748f * 0.1f;
		const bool bSupportDefaultSSSProfile = true;
		#endif

		// Legacy material uses Subsurface color as transmission 'tinting', but we can represent that with a single layer. So instead we take 
		// the max color of SubSurfaceColor & BaseColor
		LegacySlab.BaseColor = max(SubSurfaceColor, BaseColor); 
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.SSSMFP = max(1e-05f, MFPInCm); // Ensure the MFP is not null to force the material as have SSS
		LegacySlab.bSupportDefaultSSSProfile = bSupportDefaultSSSProfile;
		LegacySlab.SSSProfileID = 0;
		LegacySlab.SSSMFPScale = 1.f;
		LegacySlab.SSSPhaseAniso = 0.93f;
		LegacySlab.SSSType = SSS_TYPE_DIFFUSION;
		LegacySlab.Emissive = Emissive;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_SUBSURFACE_PROFILE
	if (ShadingModel == SHADINGMODELID_SUBSURFACE_PROFILE)
	{
		LegacySlab.BaseColor = BaseColor;
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		// Only convert material to SSS profile if opacity threshold is sufficient
		#if SUBSURFACE_PROFILE_OPACITY_THRESHOLD
		if (Opacity > SSSS_OPACITY_THRESHOLD_EPS)
		#endif
		{
			LegacySlab.SSSProfileID = SubSurfaceProfileId;
			LegacySlab.SSSMFPScale = Opacity;
			LegacySlab.SSSPhaseAniso = 0.93f;
			LegacySlab.SSSType = SSS_TYPE_DIFFUSION;	//  SSS_TYPE_DIFFUSION_PROFILE will be selected in InternalGetSubstrateSlabBSDF based on SSSProfileID and SSSMFPScale.
		}
		LegacySlab.Emissive = Emissive;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_CLEAR_COAT
	if (ShadingModel == SHADINGMODELID_CLEAR_COAT)
	{
		// When using SUBSTRATE_GBUFFER_FORMAT, data are stored into fixed GBuffer layout which does not contain bit for bUseBottomNormal
		// In this case we always force bUseBottomNormal if CLEAR_COAT_BOTTOM_NORMAL is enabled
		#if SUBSTRATE_GBUFFER_FORMAT == 0
		const bool bUseBottomNormal = CLEAR_COAT_BOTTOM_NORMAL;
		#else
		const bool bUseBottomNormal = CLEAR_COAT_BOTTOM_NORMAL && any(RawClearCoatNormal != RawNormal);
		#endif

		LegacySlab.BaseColor = BaseColor;
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.SSSProfileID = 0.f;
		LegacySlab.SSSMFP = Zeros;
		LegacySlab.SSSMFPScale = 0.f;
		LegacySlab.SecondRoughnessWeight = ClearCoat;
		LegacySlab.SecondRoughness = ClearCoatRoughness;
		LegacySlab.SecondRoughnessAsSimpleClearCoat = ClearCoat > 0.0f || bUseBottomNormal; // Haziness as simple clear coat layer,  We need to maintain it if bottom normal are used to avoid any visual pop
		LegacySlab.ClearCoatUseBottomNormal = bUseBottomNormal;
		LegacySlab.ClearCoatBottomNormal = RawClearCoatNormal;
		LegacySlab.Emissive = Emissive;
		LegacySlab.Weight = 1.f;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif

	#if MATERIAL_SHADINGMODEL_TWOSIDED_FOLIAGE
	if (ShadingModel == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		// Set a thickness that will enabled the thin lighting model (corresponding to the legacy two-sided lighting model)
		const float Thickness = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
		const float3 MFPInCm = TransmittanceToMeanFreePath(SubSurfaceColor /*TransmittanceColor*/, Thickness * CENTIMETER_TO_METER) * METER_TO_CENTIMETER;

		LegacySlab.BaseColor = BaseColor;
		LegacySlab.Specular = Specular;
		LegacySlab.Metallic = Metallic;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.SSSProfileID = 0.f;
		LegacySlab.SSSMFP = max(1e-05f, MFPInCm); // Ensure the MFP is not null to force the material as have SSS
		LegacySlab.SSSMFPScale = 1.0f;
		LegacySlab.SSSPhaseAniso = OneMinusOpacity; // Opaque-Thick: isotropic phase function, Thin: forward phase scattering function
		LegacySlab.Emissive = Emissive;
		LegacySlab.SSSType = SSS_TYPE_TWO_SIDED_WRAP;
		LegacySlab.Thickness = Thickness;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif
	
	#if MATERIAL_SHADINGMODEL_CLOTH
	if (ShadingModel == SHADINGMODELID_CLOTH)
	{
		LegacySlab.UseMetalness = false;
		LegacySlab.DiffuseAlbedo = ComputeDiffuseAlbedo(BaseColor, Metallic);
		LegacySlab.F0 = ComputeF0(Specular, BaseColor, Metallic);
		LegacySlab.F90 = 1;
		// Mask by cloth amount to emulate the old behavior where the cloth lobe gradually replaces the spec lobe depending on FuzzAmount
		LegacySlab.F0 *= 1 - ClearCoat;
		LegacySlab.Roughness = Roughness;
		LegacySlab.Anisotropy = Anisotropy;
		LegacySlab.SSSProfileID = 0.f;
		LegacySlab.SSSMFP = 0.f;
		LegacySlab.SSSMFPScale = 0.f;
		LegacySlab.Emissive = Emissive;
		LegacySlab.FuzzAmount = ClearCoat;
		LegacySlab.FuzzColor = SubSurfaceColor;

		FinalWeight = 1.0f;
#if SUBSTRATE_LEGACYCONVERT_OUTPUT_OPACITY
		FinalWeight = Opacity;
#endif
	}
	else
	#endif
	
	#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT
	if (ShadingModel == SHADINGMODELID_THIN_TRANSLUCENT)	
	{
		const float Thickness = DefaultThickness;

		const float3 TransColor = lerp(ThinTranslucentTransmittanceColor, Zeros, Opacity);
		const float3 MFP = TransmittanceToMeanFreePath(TransColor, Thickness * CENTIMETER_TO_METER) * METER_TO_CENTIMETER;
		const float F0Dieletrict = DielectricSpecularToF0(Specular);

		const float3 TopF0     = lerp(F0Dieletrict, BaseColor, Metallic);
		const float3 TopAlbedo = BaseColor * (1.f - Metallic);

		const float3 BotF0     = F0Dieletrict;
		const float3 BotAlbedo = Zeros;

		LegacySlab.DiffuseAlbedo	= lerp(BotAlbedo, TopAlbedo, Opacity);
		LegacySlab.SSSMFP			= MFP;
		LegacySlab.F0				= lerp(BotF0,     TopF0,     Opacity);
		LegacySlab.F90				= Ones;
		LegacySlab.UseMetalness		= false;
		LegacySlab.Roughness 		= Roughness;
		LegacySlab.Anisotropy 		= 0.f;
		LegacySlab.SSSProfileID 	= 0.f;
		LegacySlab.SSSMFPScale 		= 1.f;
		LegacySlab.Emissive 		= Emissive;
		LegacySlab.Thickness 		= Thickness;
		LegacySlab.SharedLocalBasisIndex = SharedLocalBasisIndex;
		LegacySlab.SSSType			= SSS_TYPE_SIMPLEVOLUME;	// This translucent explicitely use this mode to get colored translucency from SSSMFP.

		// Opacity on the root node is the coverage of the disney model on top of the translucent part.
		// Thin translucency model have a special surface coverage input.
		FinalWeight = ThinTranslucentSurfaceCoverage;
	}
	else
	#endif
	{
		// base case of else-if chain above
	}

	// Fixed layer structure for helping compiler to unroll and optimize shader
	FSubstrateData Slab = CreateLegacySlab(LegacySlab, SharedLocalBasisTypes);
#if SUBSTRATE_INLINE_SHADING
	Slab.InlinedBSDF.Coverage = LegacySlab.Weight * FinalWeight;	//////////////////// Remove FinalWeight? Do that in CreateLegacySlab?
#endif

	return Slab;
}

// Convert legacy shading models - Static
// This function is for static single shading model (i.e., known at shader compilation time). 
// It reuses the dynamic version for most part, but for special node like Unlit/Hair/Water, 
// we use the dedicated node
FSubstrateData SubstrateConvertLegacyMaterialStatic(
	FSubstratePixelFootprint PixelFootprint,
	float3 BaseColor, float Specular, float Metallic,
	float Roughness, float Anisotropy,
	float3 SubSurfaceColor, uint SubSurfaceProfileId,
	float ClearCoat, float ClearCoatRoughness,
	float3 Emissive,
	float Opacity,
	float3 ThinTranslucentTransmittanceColor,
	float ThinTranslucentSurfaceCoverage,
	float3 WaterScatteringCoefficients, float3 WaterAbsorptionCoefficients, float WaterPhaseG, float3 ColorScaleBehindWater,
	uint ShadingModel,
	float3 RawNormal,
	float3 RawTangent,
	float3 RawClearCoatNormal,
	float3 RawCustomTangent,
	uint SharedLocalBasisIndex,
	uint ClearCoatBottomNormal_SharedLocalBasisIndex,
	inout uint SharedLocalBasisTypes)
{
	FSubstrateData Out = GetInitialisedSubstrateData();

	const float DefaultThickness = SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM;
	const float3 Zeros = float3(0, 0, 0);
	const float3 Ones = float3(1, 1, 1);

	// We need to make sure a few values are safe for the transformations into slab input we do.
	Metallic = saturate(Metallic);
	ThinTranslucentTransmittanceColor = saturate(ThinTranslucentTransmittanceColor);
	ThinTranslucentSurfaceCoverage = saturate(ThinTranslucentSurfaceCoverage);
	Opacity = saturate(Opacity);

#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE
	// Opacity is applied using the "Opacity Override" pin on the root node when using alpha composite (pre multiplied alpha).
	// This is needed because some material might have the pin plugged in and the blending mode overriden on the material instance.
	Opacity = 1.0f;
#endif

	if (ShadingModel == SHADINGMODELID_UNLIT)
	{
		// Now we convert the legacy behavior to Substrate Coverage/Transmittance controls.
		// Note: Emissive will always be weighted later by the Unlit BSDF Coverage (see SubstrateUpdateTreeUnlit).

		const float GreyTransmittance = saturate(1.0f - Opacity);
	#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
		// Opaque materials only write emissive.
		Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
	#elif MATERIALBLENDING_ALPHACOMPOSITE
		// Uses (Luminance, black transmittance) for premultiplied alpha blending. 
		// Coverage over the background is specified through the root node input "Coverage Override".
		Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
	#elif MATERIALBLENDING_ALPHAHOLDOUT
		// The MATERIALBLENDING_ALPHAHOLDOUT case needs to be before the MATERIALBLENDING_TRANSLUCENT case, as alphaholdout material have both cases defined.
		Out = GetSubstrateUnlitBSDF(0.0f, GreyTransmittance, RawNormal);
	#elif MATERIALBLENDING_TRANSLUCENT
		// Uses (Luminance, black transmittance) for translucency blending. Opacity is controled using coverage.
		Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
		#if SUBSTRATE_INLINE_SHADING
			Out.InlinedBSDF.Coverage = Opacity; // Applied to emissive via the Substrate Tree, (see SubstrateUpdateTreeUnlit).
		#endif
	#elif MATERIALBLENDING_ADDITIVE
		// Emissive is weighted by opacity in the legacy material.
		Out = GetSubstrateUnlitBSDF(Emissive * Opacity, 1.0f, RawNormal);
	#elif MATERIALBLENDING_MODULATE
		// Setting up emissive as the transmittance color. It is not clamped in case brightening is required.
		Out = GetSubstrateUnlitBSDF(0.0f, Emissive, RawNormal);
	#else
		// Required default for some materials such as Editor UI.
		Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
	#endif
	}
	else if (ShadingModel == SHADINGMODELID_HAIR)
	{
		Out = GetSubstrateHairBSDF(
				BaseColor,	   // BaseColor
				Metallic,	   // Scatter
				Specular,	   // Specular
				Roughness,	   // Roughness
				ClearCoat,	   // Backlit
				Emissive,	   // EmissiveColor
				false,	       // bComplexTransmittance
				SharedLocalBasisIndex);
	}
	else if (ShadingModel == SHADINGMODELID_EYE)
	{
		const float IrisMask = ClearCoat;
		const float IrisDistance = ClearCoatRoughness;

		float3 IrisNormal = RawNormal;
		float3 IrisPlaneNormal = RawNormal;
		GetEyeNormals(IrisMask, IrisDistance, RawNormal, RawClearCoatNormal, RawCustomTangent, IrisNormal, IrisPlaneNormal);

		Out = GetSubstrateEyeBSDF(
				BaseColor,	         // DiffuseAlbedo
				Roughness,	         // Roughness
				IrisMask,	         // IrisMask
				IrisDistance,        // IrisDistance
				IrisNormal,	         // IrisNormal
				IrisPlaneNormal,	 // IrisPlaneNormal
				SubSurfaceProfileId, // SSS profile
				Emissive,	         // EmissiveColor
				SharedLocalBasisIndex);
	}
	else if (ShadingModel == SHADINGMODELID_SINGLELAYERWATER)
	{
		const float3 WaterExtinction= WaterScatteringCoefficients + WaterAbsorptionCoefficients;
		const float3 WaterAlbedo	= WaterScatteringCoefficients / WaterExtinction;

		Out = GetSubstrateSingleLayerWaterBSDF(
				BaseColor, 				// BaseColor
				Metallic, 				// Metallic 
				Specular, 				// Specular
				Roughness, 				// Roughness 
				Emissive, 				// Emissive 
				Opacity,				// TopMaterialOpacity
				WaterAlbedo, 			// WaterAlbedo
				WaterExtinction, 		// WaterExtinction
				WaterPhaseG, 			// WaterPhaseG
				ColorScaleBehindWater,	// ColorScaleBehindWater
				SharedLocalBasisIndex);
	}
	else
	{
		Out = SubstrateConvertLegacyMaterialDynamic(
			PixelFootprint,
			BaseColor, Specular, Metallic,
			Roughness, Anisotropy,
			SubSurfaceColor, SubSurfaceProfileId,
			ClearCoat, ClearCoatRoughness,
			Emissive,
			Opacity,
			ThinTranslucentTransmittanceColor,
			ThinTranslucentSurfaceCoverage,
			WaterScatteringCoefficients, WaterAbsorptionCoefficients, WaterPhaseG, ColorScaleBehindWater,
			ShadingModel,
			RawNormal,
			RawTangent,
			RawClearCoatNormal,
			RawCustomTangent,
			SharedLocalBasisIndex,
			ClearCoatBottomNormal_SharedLocalBasisIndex,
			SharedLocalBasisTypes);
	}

	return Out;
}

FSubstrateData SubstrateCreateUIMaterial(
	float3 Emissive,
	float Opacity)
{
	// We create a UI material using a Unlit Substrate BSDF. And we adapt to the blending mode in order to respect legacy behavior.
	FSubstrateData Out = GetInitialisedSubstrateData();

	const float3 RawNormal = float3(0, 0, 1);	// Unused in this case.
	Opacity = saturate(Opacity);
	const float GreyTransmittance = 1.0f - Opacity;

#if MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED
	// Opaque materials only write emissive.
	Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
#elif MATERIALBLENDING_ALPHACOMPOSITE
	// Uses (Luminance, black transmittance) for premultiplied alpha blending. 
	// Coverage over the background is specified through the root node input "Coverage Override".
	Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
#elif MATERIALBLENDING_ALPHAHOLDOUT
	// The MATERIALBLENDING_ALPHAHOLDOUT case needs to be before the MATERIALBLENDING_TRANSLUCENT case, as alphaholdout material have both cases defined.
	Out = GetSubstrateUnlitBSDF(0.0f, GreyTransmittance, RawNormal);
#elif MATERIALBLENDING_TRANSLUCENT
	// Uses (Luminance, black transmittance) for translucency blending. Opacity is controled using coverage.
	Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
#if SUBSTRATE_INLINE_SHADING
	Out.InlinedBSDF.Coverage = Opacity; // Applied to emissive via the Substrate Tree, (see SubstrateUpdateTreeUnlit).
#endif
#elif MATERIALBLENDING_ADDITIVE
	// Emissive is weighted by opacity in the legacy material.
	Out = GetSubstrateUnlitBSDF(Emissive * Opacity, 1.0f, RawNormal);
#elif MATERIALBLENDING_MODULATE
	// Setting up emissive as the transmittance color. It is not clamped in case brightening is required.
	Out = GetSubstrateUnlitBSDF(0.0f, Emissive, RawNormal);
#else
	// Required default for some materials such as Editor UI.
	Out = GetSubstrateUnlitBSDF(Emissive, 0.0f, RawNormal);
#endif

	return Out;
}
