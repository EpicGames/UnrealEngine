// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef SUBSTRATE_ENABLED
	#define SUBSTRATE_ENABLED 1
	#error SUBSTRATE_ENABLED needs to be defined
#endif

#if SUBSTRATE_ENABLED
#ifdef USE_SUBSTRATE_FORWARD_LIGHTING_COMMON

#if USE_LIGHT_FUNCTION_ATLAS
#include "../LightFunctionAtlas/LightFunctionAtlasCommon.usf"
#endif

// Used by non-mobile path
half4 GetPrecomputedShadowFactors(FSubstratePixelHeader SubstratePixelHeader, float3 TranslatedWorldPosition, bool bIsDirLight) 
{
	// PrecomputedShadowFactors are not supported on mobile at the moment
	float4 OutPrecomputedShadowFactors = SubstratePixelHeader.HasZeroPrecShadowMask() ? 0.0f : 1.0f;
#if TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING && !SHADING_PATH_MOBILE
	if (bIsDirLight)
	{
		OutPrecomputedShadowFactors.x = ComputeDirectionalLightStaticShadowing(TranslatedWorldPosition).x; // Is that actually correct for forward & deferred?

	}
	else
#endif
	{

		OutPrecomputedShadowFactors.x = 1;
	}

	return half4(OutPrecomputedShadowFactors);
}

// Common function for forward lighting per pixel using light data
float3 SubstrateForwardLightingCommon(
	float Dither,
	FSubstrateIntegrationSettings Settings,
	FDeferredLightData LightData,
	float3 ToLight,
	float LightMask,
	float4 LightAttenuation,
	FRectTexture RectTexture,
	uint LightChannelMask,
	uint PrimitiveLightingChannelMask,
	half4 PrecomputedShadowFactors,
	float3 TranslatedWorldPosition,
	float SceneDepth,
	float3 BSDFColoredVisibility,
	FSubstratePixelHeader SubstratePixelHeader,
	FSubstrateBSDFContext SubstrateBSDFContext,
	inout FSubstrateEvaluateResult BSDFEvaluate)
{
	float3 Color = 0.0;

	if (LightMask > 0.0)
	{
		// Evaluate the ShadowTerm that can then be used when integrating the lighting
		FShadowTerms ShadowTerms = { SubstrateGetAO(SubstratePixelHeader), 1.0, 1.0, InitHairTransmittanceData() };

		const uint FakeShadingModelID = 0;
		const float FakeContactShadowOpacity = 1.0f;
		GetShadowTerms(SceneDepth, PrecomputedShadowFactors, FakeShadingModelID, FakeContactShadowOpacity,
			LightData, TranslatedWorldPosition, ToLight, LightAttenuation, Dither, ShadowTerms);

		float Roughness = SubstrateGetBSDFRoughness(SubstrateBSDFContext.BSDF);
		FAreaLight AreaLight = InitAreaLight();
		BRANCH
		if (ShadowTerms.SurfaceShadow + ShadowTerms.TransmissionShadow > 0)
		{
			BSDFEvaluate = (FSubstrateEvaluateResult)0;

		#if NON_DIRECTIONAL_DIRECT_LIGHTING
			float Lighting;
			if (LightData.bRectLight)
			{
				FRect Rect = GetRect(ToLight, LightData);
				if (!IsRectVisible(Rect))
				{
					LightMask = 0.0f; // Rect light can be non visible due to barn door occlusion
				}
				AreaLight = CreateAreaLight(Roughness, SubstrateBSDFContext.N, SubstrateBSDFContext.V, Rect, RectTexture);
				Lighting = IntegrateLight(Rect);

				// We must have the evaluate inside the if due to the rectlight texture: it must be now be ambiguous which texture is going ot be used.
				// After te compilation, a local resource must map to a unique global resource (the default or the actual rect light texture).
				BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_RECT);
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
				AreaLight = CreateAreaLight(Roughness, SubstrateBSDFContext.N, SubstrateBSDFContext.V, Capsule, LightData.bInverseSquared);
				Lighting = IntegrateLight(Capsule, LightData.bInverseSquared);

				BRANCH
				if(IsAreaLight(AreaLight))
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_CAPSULE);
				}
				else
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_PUNCTUAL_LIGHT);
				}
			}

			FLATTEN
			if (LightChannelMask & PrimitiveLightingChannelMask)
			{
				float3 DiffuseLuminance = Diffuse_Lambert(BSDFEvaluate.DiffuseColor) * Lighting;
				const float3 LightCommonMultiplier = LightData.Color * LightMask;
				Color += DiffuseLuminance * LightCommonMultiplier * BSDFColoredVisibility;
			}
		#else
			if (LightData.bRectLight)
			{
				FRect Rect = GetRect(ToLight, LightData);
				if (!IsRectVisible(Rect))
				{
					LightMask = 0.0f; // Rect light can be non visible due to barn door occlusion
				}
				AreaLight = CreateAreaLight(Roughness, SubstrateBSDFContext.N, SubstrateBSDFContext.V, Rect, RectTexture);

				// We must have the evaluate inside the if due to the rectlight texture: it must be now be ambiguous which texture is going ot be used.
				// After te compilation, a local resource must map to a unique global resource (the default or the actual rect light texture).
				BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_RECT);
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
				AreaLight = CreateAreaLight(Roughness, SubstrateBSDFContext.N, SubstrateBSDFContext.V, Capsule, LightData.bInverseSquared);
				
				BRANCH
				if(IsAreaLight(AreaLight))
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_AREA_LIGHT_CAPSULE);
				}
				else
				{
					BSDFEvaluate = SubstrateEvaluateBSDFCommon(SubstrateBSDFContext, ShadowTerms, AreaLight, Settings, INTEGRATION_PUNCTUAL_LIGHT);
				}
			}

			FLATTEN
			if (LightChannelMask & PrimitiveLightingChannelMask)
			{
				float3 DiffuseLuminance = BSDFEvaluate.IntegratedDiffuseValue * LightData.DiffuseScale;
				float3 SpecularLuminance = BSDFEvaluate.IntegratedSpecularValue * LightData.SpecularScale;
				float3 LightCommonMultiplier = LightData.Color * LightMask;

			#if USE_LIGHT_FUNCTION_ATLAS
				LightCommonMultiplier *= GetLocalLightFunctionCommon(TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
			#endif

				Color += (DiffuseLuminance + SpecularLuminance) * LightCommonMultiplier * BSDFColoredVisibility;
			}
		#endif
		}
	}

	return Color;
}

#endif // USE_SUBSTRATE_FORWARD_LIGHTING_COMMON



#ifdef USE_SUBSTRATE_ENV_LIGHTING_COMMON

// USE_DEFAULT_ENV_LIGHTING_INPUT is use to allow lighting input override
#ifndef USE_DEFAULT_ENV_LIGHTING_INPUT
#define USE_DEFAULT_ENV_LIGHTING_INPUT 1
#endif 

#if USE_DEFAULT_ENV_LIGHTING_INPUT
// Diffuse input lighting (sky)
float3 GetEnvDiffuseLighting(float3 InBentNormal)
{
	return GetSkySHDiffuse(InBentNormal) * View.SkyLightColor.rgb;
}

// Specular input lighting (reflection capture + sky)
float3 GetEnvSpecularLighting(
	float CompositeAlpha,
	float3 TranslatedWorldPosition,
	float3 SpecularDirection,
	float SpecularSafeRoughness,
	float IndirectIrradiance,
	float IndirectSpecularOcclusion,
	float3 ExtraIndirectSpecular,
	uint NumCulledReflectionCaptures,
	uint CaptureDataStartIndex)
{
	const bool bCompositeSkylight = true;
	return CompositeReflectionCapturesAndSkylightTWS(
		CompositeAlpha,
		TranslatedWorldPosition,
		SpecularDirection,
		SpecularSafeRoughness,
		IndirectIrradiance,
		IndirectSpecularOcclusion,
		ExtraIndirectSpecular,
		NumCulledReflectionCaptures,
		CaptureDataStartIndex,
		0,
		bCompositeSkylight);
}
#endif // USE_DEFAULT_ENV_LIGHTING_INPUT

void SubstrateEnvLightingCommon(
	in FSubstrateEnvLightResult SubstrateEnvLight,
	in FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion,
	in FSubstrateBSDFContext SubstrateBSDFContext,
	in FSubstrateBSDF BSDF,
	in float3 BentNormal,
	in float3 BSDFThroughput,
	in uint CaptureDataStartIndex,
	in uint NumCulledReflectionCaptures,
	in float ScreenAmbientOcclusion,
	in float CloudVolumetricAOShadow,
	in float TopLayerSpecularContributionFactor,
	in float3 TranslatedWorldPosition,
	in float CombinedScreenAndMaterialAO,
	inout float3 DiffuseLighting,
	inout float3 SpecularLighting)
{

	// Diffuse component
	DiffuseLighting = 0;
#if ENABLE_DYNAMIC_SKY_LIGHT || ENABLE_SIMPLE_ENV_LIGHT
	const bool bProcessFrontFaceDiffuse = any(SubstrateEnvLight.DiffuseWeight > 0.0f);
	const bool bProcessBackFaceDiffuse = any(SubstrateEnvLight.DiffuseBackFaceWeight > 0.0f);
	if (bProcessFrontFaceDiffuse || bProcessBackFaceDiffuse)
	{
		// Compute the common sky visibility factors
		#if ENABLE_DYNAMIC_SKY_LIGHT
		FSkyLightVisibilityData SkyVisData = GetSkyLightVisibilityData(SubstrateBSDFContext.N, SubstrateBSDFContext.N, SubstrateIrradianceAndOcclusion.MaterialAO, ScreenAmbientOcclusion, BentNormal);
		#else
		FSkyLightVisibilityData SkyVisData = (FSkyLightVisibilityData)0;
		SkyVisData.SkyDiffuseLookUpMul		= CombinedScreenAndMaterialAO;
		SkyVisData.SkyDiffuseLookUpAdd		= 0.f;
		SkyVisData.SkyDiffuseLookUpNormal	= BentNormal;
		#endif

		if (bProcessFrontFaceDiffuse)
		{
			// Finally sample the sky diffuse contribution (spherical harmonic, Lambert BRDF)
			float3 DiffuseLookup = GetEnvDiffuseLighting(SkyVisData.SkyDiffuseLookUpNormal);
			// And accumulate
			// Note: Use diffuse directional albedo (i.e., DiffuseWeight) as first order approximation for env. integration (SUBSTRATE_TODO instead compute SH coefficients for Chan)
			DiffuseLighting = CloudVolumetricAOShadow * BSDFThroughput * (SkyVisData.SkyDiffuseLookUpMul * DiffuseLookup + SkyVisData.SkyDiffuseLookUpAdd) * SubstrateEnvLight.DiffuseWeight;
		}
		if (bProcessBackFaceDiffuse)
		{
			// We do not evaluate back face sky light visibility data because all the data we have is for the front face only.  This could be evaluated at some cost.
			// However, we do apply SkyVisData.SkyDiffuseLookUpMul for scaling consistency.

			// Finally sample the sky diffuse contribution (spherical harmonic, Lambert BRDF) along the opposite normal direction
			float3 DiffuseLookup = GetEnvDiffuseLighting(-SkyVisData.SkyDiffuseLookUpNormal);
			// And accumulate
			// Note: Use diffuse directional albedo (i.e., DiffuseWeight) as first order approximation for env. integration (SUBSTRATE_TODO instead compute SH coefficients for Chan)
			DiffuseLighting += CloudVolumetricAOShadow * BSDFThroughput * (SkyVisData.SkyDiffuseLookUpMul * DiffuseLookup) * SubstrateEnvLight.DiffuseBackFaceWeight;
		}
	}
#endif // ENABLE_DYNAMIC_SKY_LIGHT

	// Specular component
	const bool bIsTopLayer = BSDF_GETISTOPLAYER(BSDF);
	bool bHazeAsSimpleClearCoat = false;
	#if SUBSTRATE_FASTPATH==0
	if (BSDF_GETHASHAZINESS(BSDF))
	{
		bHazeAsSimpleClearCoat = UnpackHaziness(SLAB_HAZINESS(BSDF)).bSimpleClearCoat;
	}
	#endif

	SpecularLighting = 0;
#if SUBSTRATE_FASTPATH==0
	if (any((SubstrateEnvLight.SpecularWeight + SubstrateEnvLight.SpecularHazeWeight) > 0.0f))
#else
	if (any(SubstrateEnvLight.SpecularWeight > 0.0f))
#endif
	{
		float IndirectIrradiance = SubstrateIrradianceAndOcclusion.IndirectIrradiance;
	#if ENABLE_SKY_LIGHT && ALLOW_STATIC_LIGHTING
		BRANCH
		// Add in diffuse contribution from dynamic skylights so reflection captures will have something to mix with
		if (ReflectionStruct.SkyLightParameters.y > 0 && ReflectionStruct.SkyLightParameters.z > 0)
		{
			IndirectIrradiance += GetDynamicSkyIndirectIrradiance(BentNormal, SubstrateBSDFContext.N);
		}
	#endif

		// Compute some extra occlusion information from DFAO and sky light data
		float IndirectSpecularOcclusion = 1.0f;
		float3 ExtraIndirectSpecular = 0.0f;

	#if SUPPORT_DFAO_INDIRECT_OCCLUSION
		float IndirectDiffuseOcclusion;
		const bool bTwoSideFoliage = false;
		GetDistanceFieldAOSpecularOcclusion(BentNormal, SubstrateEnvLight.SpecularDirection, SubstrateEnvLight.SpecularSafeRoughness, bTwoSideFoliage, IndirectSpecularOcclusion, IndirectDiffuseOcclusion, ExtraIndirectSpecular);
		// Apply DFAO to IndirectIrradiance before mixing with indirect specular
		IndirectIrradiance *= IndirectDiffuseOcclusion;
	#endif

		float RoughnessSquared = SubstrateEnvLight.SpecularSafeRoughness * SubstrateEnvLight.SpecularSafeRoughness;
		float SpecularOcclusion = GetSpecularOcclusion(SubstrateBSDFContext.SatNoV, RoughnessSquared, CombinedScreenAndMaterialAO); // Do not mul with IndirectSpecularOcclusion here otherwise BentNormalAO will be applied twice.

		// When using bHazeAsSimpleClearCoat, the main specular lobe represents the bottom layer's specular, on which 
		// the TopLayerSpecularContribution (i.e. SSR) shouldn't be applied to
		SpecularLighting += BSDFThroughput * SubstrateEnvLight.SpecularWeight *
			GetEnvSpecularLighting(
				(bIsTopLayer && !bHazeAsSimpleClearCoat ? TopLayerSpecularContributionFactor : 1.0f) * SpecularOcclusion,
				TranslatedWorldPosition,
				SubstrateEnvLight.SpecularDirection,
				SubstrateEnvLight.SpecularSafeRoughness,
				IndirectIrradiance,
				IndirectSpecularOcclusion,
				ExtraIndirectSpecular,
				NumCulledReflectionCaptures,
				CaptureDataStartIndex);

	#if SUBSTRATE_FASTPATH==0
		if (BSDF_GETHASHAZINESS(BSDF))
		{
			SpecularLighting += BSDFThroughput * SubstrateEnvLight.SpecularHazeWeight *
				GetEnvSpecularLighting(
					(bIsTopLayer ? TopLayerSpecularContributionFactor : 1.0f) * SpecularOcclusion,
					TranslatedWorldPosition,
					SubstrateEnvLight.SpecularHazeDirection,
					SubstrateEnvLight.SpecularHazeSafeRoughness,
					IndirectIrradiance,
					IndirectSpecularOcclusion,
					ExtraIndirectSpecular,
					NumCulledReflectionCaptures,
					CaptureDataStartIndex);
		}
	#endif 
	}
}

#endif // SUBSTRATE_ENV_LIGHTING_COMMON
#endif // SUBSTRATE_ENABLED