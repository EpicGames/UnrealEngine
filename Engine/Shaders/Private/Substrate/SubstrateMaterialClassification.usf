// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "/Engine/Private/Common.ush"

#define SUBSTRATE_INLINE_SHADING 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
// Complex Special path is only supported with Format=1
#define SUBSTRATE_COMPLEXSPECIALPATH (SUBSTRATE_GBUFFER_FORMAT==1)

#include "/Engine/Private/Substrate/Substrate.ush"
#if SUBSTRATE_CLASSIFICATION_TILE_SIZE_16
#undef SUBSTRATE_TILE_SIZE
#undef SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT
#define SUBSTRATE_TILE_SIZE							16
#define SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT			4
#endif
#include "SubstrateTile.ush"

#if SUBSTRATE_GBUFFER_FORMAT==0
#include "../DeferredShadingCommon.ush"
#endif

#define GROUP_THREAD_COUNT (SUBSTRATE_TILE_SIZE * SUBSTRATE_TILE_SIZE)
#define SUBSTRATE_TILE_BITMASK_SSS (1<<SUBSTRATE_TILE_TYPE_COUNT)

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_CATEGORIZATION
int bRectPrimitive;
int2 ViewResolution;
uint MaxBytesPerPixel;
int FirstSliceStoringSubstrateSSSData;
Texture2D<SUBSTRATE_TOP_LAYER_TYPE> TopLayerTexture;
#if PERMUTATION_CMASK
Texture2D<uint> TopLayerCmaskTexture;
#endif
#if SUBSTRATE_GBUFFER_FORMAT==1
RWTexture2DArray<uint> MaterialTextureArrayUAV;
#endif

uint TileEncoding;
uint4 TileListBufferOffsets[SUBSTRATE_TILE_TYPE_COUNT];
uint GetTileListBufferOffsets(uint Type)
{
	return TileListBufferOffsets[Type].x;
}

// Indirect draw data buffer for all tile types
RWBuffer<uint> TileDrawIndirectDataBufferUAV;

RWBuffer<uint> TileListBufferUAV;

#if PERMUTATION_DECAL

Texture2D<float4> DBufferATexture;
Texture2D<float4> DBufferBTexture;
Texture2D<float4> DBufferCTexture;
Texture2D<uint> DBufferRenderMask;

SamplerState DBufferATextureSampler;
SamplerState DBufferBTextureSampler;
SamplerState DBufferCTextureSampler;

// @param BufferUV - UV space in the DBuffer textures
uint GetDBufferTargetMask(uint2 PixelPos)
{
	#if PLATFORM_SUPPORTS_RENDERTARGET_WRITE_MASK
	return DecodeRTWriteMask(PixelPos, DBufferRenderMask, 3);
	#elif PLATFORM_SUPPORTS_PER_PIXEL_DBUFFER_MASK
	uint Mask = DBufferRenderMask.Load(uint3(PixelPos, 0));
	return Mask > 0 ? 0x07 : 0x00;
	#else
	// For debug purpose:
	// return 
	//	(DBufferATexture.Load(uint3(PixelPos, 0)).a < 1.f ? 0x1 : 0x0) |
	//	(DBufferBTexture.Load(uint3(PixelPos, 0)).a < 1.f ? 0x2 : 0x0) |
	//	(DBufferCTexture.Load(uint3(PixelPos, 0)).a < 1.f ? 0x3 : 0x0) ;
	return 0x07;
	#endif
}

#endif // PERMUTATION_DECAL

#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
Texture2D<float3> OpaqueRoughRefractionTexture;
#endif // SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED

#if PERMUTATION_WAVE_OPS
groupshared uint s_TileBitmask;
#else
groupshared uint s_TileBitmask[GROUP_THREAD_COUNT];
#endif

[numthreads(SUBSTRATE_TILE_SIZE, SUBSTRATE_TILE_SIZE, 1)]
void TileMainCS(uint2 DispatchThreadId : SV_DispatchThreadID, uint LinearIndex : SV_GroupIndex, uint3 GroupId : SV_GroupID)
{
	// Init primitive index
	if (DispatchThreadId.x < SUBSTRATE_TILE_TYPE_COUNT && DispatchThreadId.y == 0)
	{
		const uint TileType = DispatchThreadId.x;
		const uint IndexCountPerInstance = bRectPrimitive > 0 ? 4 : 6;
		TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(TileType) + 0] = IndexCountPerInstance;
	}

	const uint2 PixelCoord = DispatchThreadId.xy + View.ViewRectMin.xy;
	const bool bIsValid    = all(DispatchThreadId.xy < uint2(View.ViewSizeAndInvSize.xy));
	const float2 BufferUV  = float2(PixelCoord + 0.5f) * View.BufferSizeAndInvSize.zw;

	// If CMask data are available, we use it as a coarse evaluation to know if a tile contains any data. 
	// * If the tile is entirely empty: we clear the header & SSS data
	// * If the data contains any data: we do fine grain checking, and clear header & SSS data only for needed pixels. The top layer data texture is used 
	//   to know if a pixel is valid or not (since the material header is not cleared when the Cmask permutation is used).
	uint GroupTileBitmask = 0;

#if PERMUTATION_CMASK && SUBSTRATE_GBUFFER_FORMAT==1
	// Coarse test for clearing header (& SSS data) based on CMask data
	const uint CMask = TopLayerCmaskTexture.Load(uint3(GroupId.xy, 0));
	BRANCH
	if (CMask == 0x0)
	{
		MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = 0u;
		SubstrateStoreSubsurfaceHeader(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, 0u); // This is a good clear for FSubstrateSubsurfaceHeader, and we only need to clear the header.
	}
	else
#endif
	{

	FSubstrateOpaqueRoughRefractionData OpaqueRoughRefractionData = (FSubstrateOpaqueRoughRefractionData)0;
	if (bIsValid)
	{
#if SUBSTRATE_GBUFFER_FORMAT==0
		// Control tiles using ShadingModelID
		const FGBufferData GBufferData   = GetGBufferData(BufferUV);
		if (GBufferData.ShadingModelID != SHADINGMODELID_UNLIT)
		{
			bool bHasAnisotropy = false;
			BRANCH
			if(HasAnisotropy(GBufferData.SelectiveOutputMask))
			{
				bHasAnisotropy = abs(GBufferData.Anisotropy) > 0;	// Skip this load when the material doesn't even support anisotropy.
			}

			#if SUBSTRATE_COMPLEXSPECIALPATH
			if (false)
			{
				GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL;
			}
			else 
			#endif
			if (SubstrateIsComplexMaterial(GBufferData.ShadingModelID, bHasAnisotropy))
			{
				GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEX;
			}
			else if (GBufferData.ShadingModelID == SHADINGMODELID_DEFAULT_LIT)
			{
				GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SIMPLE;
			}
			else
			{
				GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SINGLE;
			}

			// Skipped as only used for Format=0
			// if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE || GBufferData.ShadingModelID == SHADINGMODELID_EYE)
			// {
			// 	GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SSS;
			// }
		}

#else // SUBSTRATE_GBUFFER_FORMAT==0

		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), MaxBytesPerPixel);

		// Load mini header.
		const uint PackedHeader = MaterialTextureArrayUAV[uint3(PixelCoord, 0)];
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(PackedHeader, SubstrateAddressing, TopLayerTexture);

		const bool bIsSimple =  SubstratePixelHeader.IsSimpleMaterial() || SubstratePixelHeader.ClosureCount == 0; // BSDFCount == 0 ensures that non-Substrate pixel, like sky pixels, won't make a simple tile flagged as complex
		const bool bIsSingle = !SubstratePixelHeader.IsSimpleMaterial() && SubstratePixelHeader.IsSingleMaterial();
		
		if (SubstratePixelHeader.IsComplexSpecialMaterial())
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL;
		}
		else if (!bIsSingle && !bIsSimple)
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_COMPLEX;
		}
		else if (bIsSingle)
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SINGLE;
		}
		else if (bIsSimple)
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SIMPLE;
		}

		if (SubstratePixelHeader.HasSubsurface())
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SSS;
		}

	#if PERMUTATION_DECAL
		const uint DBufferResponseMask = SceneStencilTexture.Load(uint3(PixelCoord, 0)) STENCIL_COMPONENT_SWIZZLE;
		const uint DBufferTargetMask = GetDBufferTargetMask(PixelCoord);
		if (DBufferResponseMask != 0 && DBufferTargetMask != 0)
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_DECAL;
		}
	#endif

	#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		OpaqueRoughRefractionData = SubstrateUnpackOpaqueRoughRefractionData(OpaqueRoughRefractionTexture[PixelCoord]);
		if (OpaqueRoughRefractionData.OpaqueRoughRefractionEnabled > 0.0f)
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_ROUGH_REFRACT;
		}
	#endif

		// Output/Patch SSS data for legacy encoding (this allows to save ALU & bandwidth during the base pass0
		uint OptimisedLegacyMode = ((PackedHeader >> (HEADER_SINGLEENCODING_BIT_COUNT)) & HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK);
		const bool bIsLegacyWrapOrWrapThin = OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSWRAP || OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP; // Wrap and Wrap thin have same packing
		const bool bIsLegacySSSProfile = OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSPROFILE;
		if (bIsSingle && (bIsLegacyWrapOrWrapThin || bIsLegacySSSProfile))
		{
			GroupTileBitmask |= SUBSTRATE_TILE_BITMASK_SSS;
			if (bIsLegacyWrapOrWrapThin)
			{
				FSubstrateSubsurfaceHeader SSSHeader = (FSubstrateSubsurfaceHeader)0;
				if (OptimisedLegacyMode == SINGLE_OPTLEGACYMODE_SSSWRAP)
				{
					SubstrateSubSurfaceHeaderSetSSSType(SSSHeader, SSS_TYPE_WRAP);
				}
				else
				{
					SubstrateSubSurfaceHeaderSetSSSType(SSSHeader, SSS_TYPE_TWO_SIDED_WRAP);
				}

				const uint Data0 = PackedHeader;
				const uint Data2 = MaterialTextureArrayUAV[uint3(PixelCoord, 2)];
				const uint PackedSSSWOpacity7bits = BitFieldExtractU32(Data0, 5, HEADER_SINGLEENCODING_BIT_COUNT + HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT) | (BitFieldExtractU32(Data2, 2, 30) << 5);

				const float Opacity = UnpackR7(PackedSSSWOpacity7bits);
				SubstrateSubSurfaceHeaderSetWrapOpacity(SSSHeader, Opacity);
				SubstrateStoreSubsurfaceHeader(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSHeader.Bytes);
			}
			else if (bIsLegacySSSProfile)
			{
				const uint Data1 = MaterialTextureArrayUAV[uint3(PixelCoord, 1)];
				const uint Data2 = MaterialTextureArrayUAV[uint3(PixelCoord, 2)];

				uint ProfileId = Data2 & 0xFF;
				float RadiusScale = UnpackG8(Data2);

				const uint PackedDiffuse20Bits = (Data1 & 0xFFFFF);
				const float3 BaseColor = UnpackR7G7B6Gamma2(PackedDiffuse20Bits);

				FSubstrateSubsurfaceHeader SSSHeader = (FSubstrateSubsurfaceHeader)0;
				SubstrateSubSurfaceHeaderSetSSSType(SSSHeader, SSS_TYPE_DIFFUSION_PROFILE);
				SubstrateSubSurfaceHeaderSetProfile(SSSHeader, RadiusScale, ProfileId);

				FSubstrateSubsurfaceExtras SSSExtras = (FSubstrateSubsurfaceExtras)0;
				SubstrateSubsurfaceExtrasSetBaseColor(SSSExtras, BaseColor);

				SubstrateStoreSubsurfaceHeader(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSHeader.Bytes);
				SubstrateStoreSubsurfaceExtras(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, SSSExtras.Bytes);
			}
		}

		// Fine grain test for clearing based on CMask data
	#if PERMUTATION_CMASK
		// Fine grain check if clear is needed
		bool bClearHeader = false;
		BRANCH
		if (CMask > 0u && CMask < 0xF)
		{
			bClearHeader = !SubstrateIsTopLayerMaterial(TopLayerTexture.Load(uint3(PixelCoord, 0)));
		}

		// Header clear
		BRANCH
		if (bClearHeader)
		{
			MaterialTextureArrayUAV[uint3(PixelCoord, 0)] = 0u;
		}
	#endif

#endif // SUBSTRATE_GBUFFER_FORMAT==0
	}

#if SUBSTRATE_GBUFFER_FORMAT==1
	BRANCH
	if ((GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SSS) == 0)
	{
		// We must fill all the pixel which does not have subsurface scattering by default so that the SSS code is not executed where it should not.
		SubstrateStoreSubsurfaceHeader(MaterialTextureArrayUAV, FirstSliceStoringSubstrateSSSData, PixelCoord, 0u); // This is a good clear for FSubstrateSubsurfaceHeader, and we only need to clear the header.
	}
#endif

#if PERMUTATION_WAVE_OPS
	GroupTileBitmask = WaveActiveBitOr(GroupTileBitmask);

	if (WaveGetLaneCount() < SUBSTRATE_TILE_SIZE * SUBSTRATE_TILE_SIZE)
	{
		if (LinearIndex == 0)
		{
			s_TileBitmask = 0;
		}

		GroupMemoryBarrierWithGroupSync();
		
		if (WaveIsFirstLane())
		{
			uint Unused;
			InterlockedOr(s_TileBitmask, GroupTileBitmask, Unused);
		}
		
		GroupMemoryBarrierWithGroupSync();
		
		GroupTileBitmask = s_TileBitmask;
	}

#else // PERMUTATION_WAVE_OPS

	s_TileBitmask[LinearIndex] = GroupTileBitmask;

	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 32)
	{
		s_TileBitmask[LinearIndex] = s_TileBitmask[LinearIndex] | s_TileBitmask[LinearIndex + 32];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		s_TileBitmask[LinearIndex] = s_TileBitmask[LinearIndex] | s_TileBitmask[LinearIndex + 16];
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 8)
	{
		s_TileBitmask[LinearIndex] = s_TileBitmask[LinearIndex] | s_TileBitmask[LinearIndex + 8];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		s_TileBitmask[LinearIndex] = s_TileBitmask[LinearIndex] | s_TileBitmask[LinearIndex + 4];
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		s_TileBitmask[LinearIndex] = s_TileBitmask[LinearIndex] | s_TileBitmask[LinearIndex + 2];
	}
	GroupMemoryBarrierWithGroupSync();

	GroupTileBitmask = s_TileBitmask[0] | s_TileBitmask[1];
#endif // PERMUTATION_WAVE_OPS

	if (LinearIndex == 0 && GroupTileBitmask != 0)
	{
		const uint EncodedTile = SubstratePackTile(GroupId.xy, TileEncoding);
		#if SUBSTRATE_COMPLEXSPECIALPATH
		if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL) + WriteToIndex] = EncodedTile;
		}
		else 
		#endif
		if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_COMPLEX)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_COMPLEX) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_COMPLEX) + WriteToIndex] = EncodedTile;
		}
		else if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SINGLE)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_SINGLE) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_SINGLE) + WriteToIndex] = EncodedTile;
		}
		else // (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SIMPLE)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_SIMPLE) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_SIMPLE) + WriteToIndex] = EncodedTile;
		}

	#if SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
		const bool bTileContainsOpaqueRoughRefraction			= GroupTileBitmask & SUBSTRATE_TILE_BITMASK_ROUGH_REFRACT; 			
		const bool bTileContainsScreenSpaceSubsurfaceScattering	= GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SSS;
		if (bTileContainsOpaqueRoughRefraction)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_ROUGH_REFRACT) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_ROUGH_REFRACT) + WriteToIndex] = EncodedTile;
		}
		if(bTileContainsScreenSpaceSubsurfaceScattering && !bTileContainsOpaqueRoughRefraction)
		{
			uint WriteToIndex;
			InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT) + 1], 1, WriteToIndex);
			TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT) + WriteToIndex] = EncodedTile;
		}
	#endif

	#if PERMUTATION_DECAL
		if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_DECAL)
		{
			if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_COMPLEX)
			{
				uint WriteToIndex;
				InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_DECAL_COMPLEX) + 1], 1, WriteToIndex);
				TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_DECAL_COMPLEX) + WriteToIndex] = EncodedTile;
			}
			else if (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SINGLE)
			{
				uint WriteToIndex;
				InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_DECAL_SINGLE) + 1], 1, WriteToIndex);
				TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_DECAL_SINGLE) + WriteToIndex] = EncodedTile;
			}
			else // (GroupTileBitmask & SUBSTRATE_TILE_BITMASK_SIMPLE)
			{
				uint WriteToIndex;
				InterlockedAdd(TileDrawIndirectDataBufferUAV[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(SUBSTRATE_TILE_TYPE_DECAL_SIMPLE) + 1], 1, WriteToIndex);
				TileListBufferUAV[GetTileListBufferOffsets(SUBSTRATE_TILE_TYPE_DECAL_SIMPLE) + WriteToIndex] = EncodedTile;
			}
		}
	#endif
	}

	}
}
#endif // SHADER_TILE_CATEGORIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_MATERIAL_TILE_PREPARE_ARGS

Buffer<uint>   TileDrawIndirectDataBuffer;
RWBuffer<uint> TileDispatchIndirectDataBuffer;

[numthreads(32, 1, 1)]
void ArgsMainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint TileType = DispatchThreadId.x;
	if (TileType < SUBSTRATE_TILE_TYPE_COUNT)
	{
		// We could have more than 65k tile in particular with complex multi-layer closure covering full 
		TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(TileType) + 0] = TileDrawIndirectDataBuffer[GetSubstrateTileTypeDrawIndirectArgOffset_DWord(TileType) + 1];
		TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(TileType) + 1] = 1;
		TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(TileType) + 2] = 1;
	}
}

#endif // SHADER_MATERIAL_TILE_PREPARE_ARGS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_CLOSURE_TILE_PREPARE_ARGS

int2 TileCount_Primary;

Buffer<uint>   TileDrawIndirectDataBuffer;
RWBuffer<uint> TileDispatchIndirectDataBuffer;
RWBuffer<uint> TileDispatchPerThreadIndirectDataBuffer;
RWBuffer<uint> TileRaytracingIndirectDataBuffer;

void WriteArgs(uint InTileCount, uint OutOffset)
{
	const uint DispatchX = min(InTileCount, uint(TileCount_Primary.x));
	const uint DispatchY = DivideAndRoundUp(InTileCount, TileCount_Primary.x);

	TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 0] = DispatchX;
	TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 1] = DispatchY;
	TileDispatchIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 2] = 1;
		
	TileDispatchPerThreadIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 0] = DivideAndRoundUp(InTileCount, SUBSTRATE_TILE_SIZE * SUBSTRATE_TILE_SIZE);
	TileDispatchPerThreadIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 1] = 1;
	TileDispatchPerThreadIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 2] = 1;

	// Indirect raytracing args are mapped on ray count. Each tile is expended into rays.
	const uint RayDispatchX = min(InTileCount, uint(TileCount_Primary.x)) * SUBSTRATE_TILE_SIZE;
	const uint RayDispatchY = DivideAndRoundUp(InTileCount, TileCount_Primary.x) * SUBSTRATE_TILE_SIZE;

	TileRaytracingIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 0] = RayDispatchX;
	TileRaytracingIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 1] = RayDispatchY;
	TileRaytracingIndirectDataBuffer[GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(OutOffset) + 2] = 1;
}

[numthreads(3, 1, 1)]
void ArgsMainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint WriteOffset = DispatchThreadId.x;
	if (WriteOffset < 3)
	{
		const uint TileCount = TileDrawIndirectDataBuffer[0].x;

		uint TileCounts[3];
		TileCounts[0] = TileCount;						// Downsample factor=1
		TileCounts[1] = DivideAndRoundUp4(TileCount);   // Downsample factor=2 -  4 subtiles per per 8x8 tile
		TileCounts[2] = DivideAndRoundUp16(TileCount);  // Downsample factor=3 - 16 subtiles per per 8x8 tile
		WriteArgs(TileCounts[WriteOffset], WriteOffset);
	}
}

#endif // SHADER_CLOSURE_TILE_PREPARE_ARGS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_CLOSURE_TILE

int2 ViewResolution;
uint MaxBytesPerPixel;
uint TileSizeLog2;

int2 TileCount_Primary;

Texture2D<SUBSTRATE_TOP_LAYER_TYPE> TopLayerTexture;
Texture2DArray<uint> MaterialTextureArray;

Buffer<uint> TileListBuffer;
uint TileListBufferOffset;
uint TileEncoding;

RWTexture2D<uint> RWClosureOffsetTexture;
RWBuffer<uint>    RWClosureTileCountBuffer;
RWBuffer<uint>    RWClosureTileBuffer;

#if !PERMUTATION_WAVE_OPS
groupshared uint s_TileClosureCount[GROUP_THREAD_COUNT];
#endif

#if PERMUTATION_WAVE_OPS && COMPILER_SUPPORTS_WAVE_SIZE
WAVESIZE(64) // PERMUTATION_WAVE_OPS is true only when wave>=64 are available
#endif
[numthreads(SUBSTRATE_TILE_SIZE, SUBSTRATE_TILE_SIZE, 1)]
void ClosureTileMainCS(uint2 GroupThreadId : SV_GroupThreadID, uint2 GroupId : SV_GroupID, uint LinearIndex : SV_GroupIndex)
{
	const uint2 TileCoord = SubstrateUnpackTile(TileListBuffer[TileListBufferOffset + GroupId.x], TileEncoding);
	uint2 PixelCoord = TileCoord * SUBSTRATE_TILE_SIZE + GroupThreadId;
	const bool bIsInViewRect = all(PixelCoord < uint2(View.ViewRectMinAndSize.zw));
	PixelCoord += View.ViewRectMinAndSize.xy;

	uint ClosureCount = 0;
	if (bIsInViewRect)
	{
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(MaterialTextureArray, SubstrateAddressing, TopLayerTexture);
		ClosureCount = min(SubstratePixelHeader.ClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);

		if (ClosureCount > 0)
		{
			FSubstrateClosureOffset Offsets = (FSubstrateClosureOffset)0;
			Offsets.ClosureCount = ClosureCount;

			UNROLL_N(SUBSTRATE_MATERIAL_CLOSURE_COUNT)
			for (uint ClosureIndex = 0; ClosureIndex < ClosureCount; ++ClosureIndex)
			{
				Offsets.ClosureOffsets[ClosureIndex] = SubstrateAddressing.CurrentIndex;
				UnpackSubstrateBSDFIn(MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
			}

			RWClosureOffsetTexture[PixelCoord] = PackClosureOffset(Offsets);
		}
	}

#if PERMUTATION_WAVE_OPS

	const uint TileClosureCount = WaveActiveMax(ClosureCount);

#else // PERMUTATION_WAVE_OPS

	s_TileClosureCount[LinearIndex] = ClosureCount;

	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 32)
	{
		s_TileClosureCount[LinearIndex] = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 32]);
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 16)
	{
		s_TileClosureCount[LinearIndex] = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 16]);
	}
	GroupMemoryBarrierWithGroupSync();

	if (LinearIndex < 8)
	{
		s_TileClosureCount[LinearIndex] = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 8]);
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 4)
	{
		s_TileClosureCount[LinearIndex] = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 4]);
	}
	GroupMemoryBarrierWithGroupSync();
	if (LinearIndex < 2)
	{
		s_TileClosureCount[LinearIndex] = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 2]);
	}
	GroupMemoryBarrierWithGroupSync();

	const uint TileClosureCount = max(s_TileClosureCount[LinearIndex], s_TileClosureCount[LinearIndex + 1]);

#endif // PERMUTATION_WAVE_OPS

	#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
	if (LinearIndex == 0)
	{
		if (TileClosureCount > 1)
		{
			// Store only tile data for Closure[1..X]. Closure[0] is implicity stored into the first layer
			uint StoreIndex = 0;
			InterlockedAdd(RWClosureTileCountBuffer[0], TileClosureCount - 1, StoreIndex);

			FSubstrateClosureTile Tile;
			Tile.TileCoord = TileCoord;
			Tile.ClosureCount = TileClosureCount;
			for (uint ClosureIndex = 1; ClosureIndex < TileClosureCount; ++ClosureIndex)
			{
				Tile.ClosureIndex = ClosureIndex;
				RWClosureTileBuffer[StoreIndex + ClosureIndex - 1] = PackClosureTile(Tile);
			}
		}
	}
	#endif
}
#endif // SHADER_CLOSURE_TILE
