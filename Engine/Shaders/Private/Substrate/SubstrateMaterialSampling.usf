// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "/Engine/Private/Common.ush"

#define SUBSTRATE_INLINE_SHADING 0
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
#define SUBSTRATE_COMPLEXSPECIALPATH 1
#include "/Engine/Private/Substrate/Substrate.ush"
#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#include "/Engine/Private/Substrate/SubstrateTile.ush"

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_SAMPLE_MATERIAL

#include "SubstrateMaterialSampling.ush"
#include "../Lumen/LumenPosition.ush"
#include "../BlueNoise.ush"

#define DEBUG_ENABLED 0
#if DEBUG_ENABLED
	#include "../ShaderPrint.ush"
#endif

uint MaxBytesPerPixel;
uint MaxClosurePerPixel;
uint bRoughDiffuse;
uint PeelLayersAboveDepth;
uint bRoughnessTracking;
uint MegaLightsStateFrameIndex;

Texture2D<SUBSTRATE_TOP_LAYER_TYPE> TopLayerTexture;
Texture2DArray<uint> MaterialTextureArray;
Texture2D<uint> ClosureOffsetTexture;
RWTexture2D<uint4> RWMaterialData;

///////////////////////////////////////////////////////////////////////////////////////////////////
// TODO merge these helper functions with path-tracer

float3 GetSimpleVolumeDiffuseColor(float3 DiffuseColor, float3 MeanFreePath)
{
	// See reference in SubstrateEvaluation.ush
	// SUBSTRATE_TODO: In the case of thin materials, we should also include a backscattering diffuse portion
	FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(DiffuseColor, MeanFreePath);
	const float DiffuseToVolumeBlend = SubstrateSlabDiffuseToVolumeBlend(PM);
	const float3 SlabDirectionalAlbedo = IsotropicMediumSlabEnvDirectionalAlbedoALU(PM);
	return lerp(DiffuseColor, SlabDirectionalAlbedo, DiffuseToVolumeBlend);
}

FBxDFEnergyTermsA ComputeSheenEnergyTerms(float NoV, float SheenRoughness)
{
	#if 1
	FBxDFEnergyTermsA Result;
	Result.E = SheenLTC_DirectionalAlbedo(NoV, SheenRoughness, View.SheenLTCTexture, View.SheenLTCSampler);
	Result.W = Result.E; // overall weight is the directional albedo
	return Result;
	#else
	return ComputeClothEnergyTermsA(SheenRoughness, NoV);
	#endif
}

float LobeColorToWeight(float3 C)
{
	// TODO: What is the best heuristic to use here?
	// Sum seems to perform slightly better sometimes, while max3 is better in others. Need to look at more cases to be sure
	// Also tried Luminance, but the result was very close to the plain sum
	#if 1
	return C.x + C.y + C.z;
	#else
	return max3(C.x, C.y, C.z);
	#endif
}

float GetSlabPDF(FSubstrateAddressing SubstrateAddressing, FSubstratePixelHeader SubstratePixelHeader, FSubstrateBSDF BSDF, float3 V_World)
{
	float Pdf = 0.0; // The goal is to predict the weight of each BSDF
	switch (BSDF_GETTYPE(BSDF))
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			const float3 WeightV = BSDF.LuminanceWeightV;
			const float3 TransmittanceN = BSDF.TransmittanceAboveAlongN;
			const float CoverageAboveAlongN = BSDF.CoverageAboveAlongN;
			const float3 MaxLobeWeight = WeightV * lerp(1.0f, TransmittanceN, CoverageAboveAlongN); // largest value LobeWeight() could return

			// Is the slab even visible at all?
			if (any(MaxLobeWeight > 0.0))
			{
				const float3 N = SubstrateGetWorldNormal(SubstratePixelHeader, BSDF, SubstrateAddressing);
				const float NoV = saturate(dot(N, V_World));

				float3 DiffuseColor = SLAB_DIFFUSEALBEDO(BSDF);
				float3 F0 = SLAB_F0(BSDF);
				float3 F90 = SLAB_F90(BSDF) * F0RGBToMicroOcclusion(F0);

				float Roughness0 = MakeRoughnessSafe(SLAB_ROUGHNESS(BSDF));
				// SUBSTRATE_TODO: Support Fresnel82?
				float3 Spec0E = ComputeGGXSpecEnergyTermsRGB(Roughness0, NoV, F0, F90).E;
				float3 Spec1E = Spec0E;
				float Roughness1 = Roughness0;
				float SpecBlend = 0.0;
				bool bUseClearCoat = false;
				if (BSDF_GETHASHAZINESS(BSDF))
				{
					const FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
					SpecBlend = Haziness.Weight;
					Roughness1 = MakeRoughnessSafe(Haziness.Roughness);
					if (Haziness.bSimpleClearCoat)
					{
						const float CLEAR_COAT_F0 = 0.04f;
						Spec1E = ComputeGGXSpecEnergyTermsRGB(Roughness1, NoV, CLEAR_COAT_F0).E;
						bUseClearCoat = true;
					}
					else
					{
						Spec1E = ComputeGGXSpecEnergyTermsRGB(Roughness1, NoV, F0, F90).E;
					}
				}
				if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_SIMPLEVOLUME)
				{
					DiffuseColor = GetSimpleVolumeDiffuseColor(DiffuseColor, SLAB_SSSMFP(BSDF));
				}

				float FuzzAmount = 0.0;
				float FuzzRoughness = 1.0;
				float3 FuzzColor = 0.0;
				if (BSDF_GETHASFUZZ(BSDF))
				{
					FuzzAmount = SLAB_FUZZ_AMOUNT(BSDF);
					FuzzColor = SLAB_FUZZ_COLOR(BSDF);
					FuzzRoughness = SLAB_FUZZ_ROUGHNESS(BSDF);
					FuzzRoughness = MakeRoughnessSafe(FuzzRoughness, SUBSTRATE_MIN_FUZZ_ROUGHNESS);
				}
							
				const float FuzzE = ComputeSheenEnergyTerms(FuzzRoughness, NoV).E;
				const float FuzzAttenuation = 1.0 - FuzzAmount * FuzzE;
				float DiffuseWeight = FuzzAttenuation;
				float3 Spec0Albedo = FuzzAttenuation * Spec0E;
				float3 Spec1Albedo = FuzzAttenuation * Spec1E;
				if (bUseClearCoat)
				{
					// clearcoat slab
					float CoatAttenuation = 1.0 - SpecBlend * Spec1E.x;
					DiffuseWeight *= CoatAttenuation * (1.0 - Luminance(Spec0E));
					Spec0Albedo *= CoatAttenuation;
					Spec1Albedo *= SpecBlend;
				}
				else
				{
					// dual-specular slab
					DiffuseWeight *= 1.0 - Luminance(lerp(Spec0E, Spec1E, SpecBlend));
					Spec0Albedo *= 1.0 - SpecBlend;
					Spec1Albedo *= SpecBlend;
				}
				// SUBSTRATE_TODO: Need to account for glass case here for lobe selection ....
				const float3 FuzzAlbedo = FuzzAmount * FuzzE * FuzzColor;
				const float3 SlabAlbedo = MaxLobeWeight * (DiffuseWeight * DiffuseColor + Spec0Albedo + Spec1Albedo + FuzzAlbedo);
				Pdf = LobeColorToWeight(SlabAlbedo);
			}
			break;
		}
		default:
		{
			// In theory none of the other slab types can be layered
			break;
		}
	}
	return Pdf;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

FSubstrateSampledMaterial GetSampledMaterial(uint2 InPixelCoord, uint ClosureIndex, float3 V, float InPDF)
{
	const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, bRoughDiffuse, PeelLayersAboveDepth, bRoughnessTracking);
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(InPixelCoord, uint2(View.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(MaterialTextureArray, SubstrateAddressing, TopLayerTexture);
	const uint ClosureCount = min(SubstratePixelHeader.ClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);

	// * Force single closure evaluation
	#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
	if (ClosureCount > 1)
	{
		const uint AddressOffset = UnpackClosureOffsetAtIndex(ClosureOffsetTexture[SubstrateAddressing.PixelCoords], ClosureIndex, SubstratePixelHeader.ClosureCount);
		SubstrateSeekClosure(SubstrateAddressing, AddressOffset);
	}
	#endif
	SubstratePixelHeader.ClosureCount = 1;

	FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);

	// Create the BSDF context
	FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, BSDF, SubstrateAddressing, V);
	const float3 BSDFThroughput = LuminanceWeight(SubstrateBSDFContext, BSDF);

	// Evaluate environment lighting
	FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);
	
	const uint BSDFType = SubstratePixelHeader.SubstrateGetBSDFType();

	FSubstrateSampledMaterial Out = (FSubstrateSampledMaterial)0;
	Out.bAllowSpatialFilter = true;
	Out.bIsValid			= true;
	Out.WorldNormal			= SubstrateGetWorldNormal(SubstratePixelHeader, BSDF, SubstrateAddressing);
	Out.WorldTangent		= SubstrateGetWorldTangent(SubstratePixelHeader, BSDF, SubstrateAddressing);
	Out.Roughness			= SubstrateGetBSDFRoughness(BSDF);
	Out.bIsSimple			= SubstratePixelHeader.IsSimpleMaterial();
	Out.bIsSingle			= SubstratePixelHeader.IsSingleMaterial();
	Out.bIsHair 			= BSDFType == SUBSTRATE_BSDF_TYPE_HAIR;
	Out.bIsSLW				= BSDFType == SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER;
	Out.bIsFirstPerson		= SubstratePixelHeader.IsFirstPerson();
	Out.bHasSecondSpecularLobe= BSDF_GETHASHAZINESS(BSDF);
	Out.bHasBackScattering	= BSDFType == SUBSTRATE_BSDF_TYPE_SLAB && SubstratePixelHeader.HasSubsurface();
	Out.ClosureIndex		= ClosureIndex;
	Out.PDF					= InPDF;
	Out.Anisotropy			= SubstrateGetBSDFAnisotropy(BSDF);
	Out.IrradianceAO		= SubstrateGetPackedIrradianceAndAO(SubstratePixelHeader);

	float3 OutDiffuseColor  = BSDFThroughput * SubstrateEnvLight.DiffuseColor;  //SubstrateEnvLight.DiffuseWeight;
	float3 OutSpecularColor = BSDFThroughput * SubstrateEnvLight.SpecularColor; //SubstrateEnvLight.SpecularWeight;
	//#if SUBSTRATE_FASTPATH==0
	if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
	{
		OutSpecularColor += BSDFThroughput * SubstrateEnvLight.SpecularHazeWeight;
	}
	//#endif 
	if (SubstrateEnvLight.bPostProcessSubsurface)
	{
		Out.bNeedsSeparateSubsurfaceLightAccumulation = true;
	}
	//#if SUBSTRATE_FASTPATH==0
	if (SubstratePixelHeader.IsComplexSpecialMaterial() && BSDF_GETHASGLINT(BSDF))
	{
		Out.bAllowSpatialFilter = false;
	}
	//#endif

	Out.DiffuseAlbedo	= OutDiffuseColor;
	Out.SpecularAlbedo  = OutSpecularColor;
	return Out;
}

#if DEBUG_ENABLED
void Print(inout FShaderPrintContext Ctx, FSubstrateSampledMaterial Out)
{
	Newline(Ctx);

	PrintLineN(Ctx, Out.WorldNormal);
	PrintLineN(Ctx, Out.WorldTangent);
	PrintLineN(Ctx, Out.Roughness);
	PrintLineN(Ctx, Out.DiffuseAlbedo);
	PrintLineN(Ctx, Out.SpecularAlbedo);
	PrintLineN(Ctx, Out.Anisotropy);
	PrintLineN(Ctx, Out.PDF);
	PrintLineN(Ctx, Out.ClosureIndex);
	PrintLineN(Ctx, Out.IrradianceAO);

	PrintLineN(Ctx, Out.bIsSimple);
	PrintLineN(Ctx, Out.bIsSingle);
	PrintLineN(Ctx, Out.bIsValid);
	PrintLineN(Ctx, Out.bIsHair);
	PrintLineN(Ctx, Out.bIsSLW);
	PrintLineN(Ctx, Out.bAllowSpatialFilter);
	PrintLineN(Ctx, Out.bNeedsSeparateSubsurfaceLightAccumulation);
	PrintLineN(Ctx, Out.bIsFirstPerson);
	PrintLineN(Ctx, Out.bHasSecondSpecularLobe);
	PrintLineN(Ctx, Out.bHasBackScattering);
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////

[numthreads(SUBSTRATE_TILE_SIZE, SUBSTRATE_TILE_SIZE, 1)]
void SubstrateMaterialSamplingCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelCoord = DispatchThreadId + View.ViewRectMinAndSize.xy;
	const bool bIsInViewRect = all(PixelCoord < uint2(View.ViewRectMinAndSize.zw));

	if (bIsInViewRect)
	{
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, bRoughDiffuse, PeelLayersAboveDepth, bRoughnessTracking);
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelCoord, uint2(View.BufferSizeAndInvSize.xy), MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(MaterialTextureArray, SubstrateAddressing, TopLayerTexture);
		const uint ClosureCount = min(SubstratePixelHeader.ClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);

		#if DEBUG_ENABLED
		FShaderPrintContext Ctx = InitShaderPrintContextAtCursor(PixelCoord, uint2(50, 50));
		Print(Ctx, TEXT("DEBUG"), FontRed); Newline(Ctx);
		PrintLineN(Ctx, PixelCoord);
		PrintLineN(Ctx, ClosureCount);
		#endif

		FSubstrateSampledMaterial Out = (FSubstrateSampledMaterial)0;
		if (ClosureCount > 0)
		{
			const float2 ScreenUV = (PixelCoord + 0.5f) * View.BufferSizeAndInvSize.zw;
			const float SceneDepth = ConvertFromDeviceZ(SceneTexturesStruct.SceneDepthTexture.Load(int3(PixelCoord, 0)).r);
			const float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromScreenUV(ScreenUV, SceneDepth);
			const float3 V = -GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);

			float SlabPdf = 1.0;
			uint ClosureIndex = 0;
			#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
			if (ClosureCount > 1)
			{
				// Uniform sampling rather directional albedo sampling seems to give better results as we can reuse neighborhood 
				// results more easily for resolving lower/less probably layers
				#define USE_UNIFORM_SAMPLING 1

				#if USE_UNIFORM_SAMPLING
				const float RandSample = ClosureCount * BlueNoiseScalar(PixelCoord, MegaLightsStateFrameIndex);
				ClosureIndex = clamp(RandSample, 0, ClosureCount-1);
				SlabPdf = 1.f / float(ClosureCount);
				#else
				float SlabCDF[SUBSTRATE_MATERIAL_CLOSURE_COUNT];
				float SlabCDFSum = 0.0;

				// Build Slabs' CDF 
				{
					UNROLL_N(SUBSTRATE_MATERIAL_CLOSURE_COUNT)
					for (uint ClosureIndex = 0; ClosureIndex < ClosureCount; ++ClosureIndex)
					{
						const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(MaterialTextureArray, SubstrateAddressing, SubstratePixelHeader);
						const float Pdf = GetSlabPDF(SubstrateAddressing, SubstratePixelHeader, BSDF, V);
						SlabCDFSum += Pdf;
						SlabCDF[ClosureIndex] = SlabCDFSum;
					}	
				}
			
				// Select Slab
				if (SlabCDFSum > 0.0)
				{
					// linear search
					float PreviousCdfValue = 0;
					float RandSample = SlabCDFSum * BlueNoiseScalar(PixelCoord, MegaLightsStateFrameIndex);
					for (ClosureIndex = 0; ClosureIndex < ClosureCount - 1; ++ClosureIndex)
					{
						if (RandSample < SlabCDF[ClosureIndex])
						{
							break;
						}
						PreviousCdfValue = SlabCDF[ClosureIndex];
					}
					SlabPdf = (SlabCDF[ClosureIndex] - PreviousCdfValue) / SlabCDFSum;

					#if DEBUG_ENABLED
					PrintLineN(Ctx, SlabPdf);
					for (uint CDFIndex = 0; CDFIndex < ClosureCount; ++CDFIndex)
					{
						PrintLineN(Ctx, SlabCDF[CDFIndex]);
					}
					#endif
				}
				#endif // USE_UNIFORM_SAMPLING
			}
			else
			#endif // SUBSTRATE_MATERIAL_CLOSURE_COUNT
			{
				ClosureIndex = 0;
				SlabPdf = 1.0;
			}

			Out = GetSampledMaterial(PixelCoord, ClosureIndex, V, SlabPdf);
			#if DEBUG_ENABLED
			Print(Ctx, Out);
			#endif
		}

		RWMaterialData[PixelCoord] = PackSampledMaterial(Out);
	}
}

#endif // SHADER_SAMPLE_MATERIAL
