// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/Substrate/Substrate.ush"

#if SUBSTRATE_ENABLED

struct FSubstrateSampledMaterial
{
	float3 WorldNormal;
	float3 WorldTangent;
	float  Roughness;
	float3 DiffuseAlbedo;
	float3 SpecularAlbedo;
	float  Anisotropy;
	float  PDF;
	uint   ClosureIndex;
	uint   IrradianceAO;

	bool bIsSimple;
	bool bIsSingle;
	bool bIsValid;
	bool bIsHair;
	bool bIsSLW;
	bool bAllowSpatialFilter;
	bool bNeedsSeparateSubsurfaceLightAccumulation;
	bool bIsFirstPerson;
	bool bHasSecondSpecularLobe;
	bool bHasBackScattering;
};

FSubstrateSampledMaterial UnpackSampledMaterial(uint4 In)
{
	const float4 DataY = UnpackRGBA8(In.y);
	const float4 DataZ = UnpackRGBA8(In.z);

	FSubstrateSampledMaterial Out;
	// Can't use last 8 bit as we store 24bit normal or 32bit normal/tangent
	Out.WorldNormal        = SubstrateUnpackNormal24(In.x); 
	Out.WorldTangent       = 0;

	Out.DiffuseAlbedo      = DataY.xyz;
	Out.Roughness          = DataY.w;

	Out.SpecularAlbedo     = DataZ.xyz;
	const uint Type        = BitFieldExtractU32(In.z, 3, 24);
	Out.bIsFirstPerson     = BitFieldExtractU32(In.z, 1, 27);
	Out.bHasSecondSpecularLobe= BitFieldExtractU32(In.z, 1, 28);
	Out.bAllowSpatialFilter= BitFieldExtractU32(In.z, 1, 29);
	Out.bHasBackScattering = BitFieldExtractU32(In.z, 1, 30);
	Out.bNeedsSeparateSubsurfaceLightAccumulation = BitFieldExtractU32(In.z, 1, 31);
	Out.bIsValid           = Type > 0;
	Out.bIsSimple          = Type == 1;
	Out.bIsSingle          = Type == 2;
	Out.bIsHair            = Type == 4;
	Out.bIsSLW             = Type == 5;

	Out.PDF                = Unpack10F(In.w);
	Out.ClosureIndex       = BitFieldExtractU32(In.w, 3, 10);
	Out.IrradianceAO       = BitFieldExtractU32(In.w, 8, 16);
	uint PackedAniso8bits  = BitFieldExtractU32(In.w, 8, 24);
	Out.Anisotropy         = UnpackR8(PackedAniso8bits & 0xFE);
	Out.Anisotropy        *= (PackedAniso8bits & 1u) ? 1.f : -1.f;
	
	if (Out.Anisotropy != 0)
	{
		SubstrateUnpackNormalAndTangent(Out.WorldNormal, Out.WorldTangent, In.x);
	}
	return Out;
}

uint4 PackSampledMaterial(FSubstrateSampledMaterial In)
{
	uint Type = 0;
	if (In.bIsSimple) { Type = 1; }
	if (In.bIsSingle) { Type = 2; }
	if (In.bIsHair)   { Type = 4; }
	if (In.bIsSLW)    { Type = 5; }
	if (In.bIsValid && Type == 0)  { Type = 3; /*bIsComplex*/ }

	uint DataX;
	if (In.Anisotropy == 0)
	{
		DataX = SubstratePackNormal24(In.WorldNormal);
	}
	else
	{
		DataX = SubstratePackNormalAndTangent(In.WorldNormal, In.WorldTangent);
	}

	const uint PackedAniso8bits = (PackR8(abs(In.Anisotropy)) & 0xFE) | (In.Anisotropy > 0 ? 1u : 0u);
	const uint DataZ = 
		((Type & 0x3)												 << 24) |
		((In.bIsFirstPerson								? 0x1 : 0x0) << 27) |
		((In.bHasSecondSpecularLobe						? 0x1 : 0x0) << 28) |
		((In.bAllowSpatialFilter						? 0x1 : 0x0) << 29) |
		((In.bHasBackScattering							? 0x1 : 0x0) << 30) |
		((In.bNeedsSeparateSubsurfaceLightAccumulation	? 0x1 : 0x0) << 31);
	const uint DataW = 
		Pack10F(In.PDF) |
		((In.ClosureIndex & 0x3u) << 10) |
		((In.IrradianceAO & 0xFF) << 16) |
		(PackedAniso8bits << 24);

	uint4 Out;
	Out.x = DataX;
	Out.y = PackRGBA8(float4(In.DiffuseAlbedo, In.Roughness));
	Out.z = PackRGBA8(float4(In.SpecularAlbedo, 0)) | DataZ;
	Out.w = DataW;
	return Out;
}

#endif