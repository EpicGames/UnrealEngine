// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define USE_SUBSTRATE_FORWARD_LIGHTING_COMMON 1
#include "SubstrateLightingCommon.ush"
#include "../LightData.ush"
#include "../MobileLightingCommon.ush"

#define FORWARD_PER_PIXEL_SHADING (FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING || MATERIAL_SHADINGMODEL_SINGLELAYERWATER)

// For debug or iteration purpose only
#ifndef SUBSTRATE_FORCE_SINGLE_LOCAL_LIGHT
#define SUBSTRATE_FORCE_SINGLE_LOCAL_LIGHT 0
#endif

// Forward declarations
void GetVolumeLightingNonDirectional(float4 AmbientLightingVector, float3 DiffuseColor, inout float3 InterpolatedLighting, out float4 VolumeLighting);
void GetVolumeLightingDirectional(float4 AmbientLightingVector, float3 DirectionalLightingVector, float3 WorldNormal, float3 DiffuseColor, float DirectionalLightingIntensity, inout float3 InterpolatedLighting, out float4 VolumeLighting);

float3 GetWorldBentNormalZero(in FMaterialPixelParameters MaterialParameters);

void GetPrecomputedIndirectLightingAndSkyLight(
	VTPageTableResult LightmapVTPageTableResult, 
	FVertexFactoryInterpolantsVSToPS Interpolants, 
	bool bEvaluateBackface, 
	half3 DiffuseDir, 
	half3 SkyLightColor, 
	out half3 OutDiffuseIndirectLighting, 
	out half3 OutSubsurfaceIndirectLighting, 
	out half OutIndirectIrradiance, 
	out half3 OutWaterDiffuseIndirectLuminance);

// This code should map to GetForwardDirectLightingSplit
float3 SubstrateMobileForwardLighting(
	uint EyeIndex,
	float4 SvPosition,
	FSubstrateIntegrationSettings Settings,
	FMobileBasePassInterpolantsVSToPS BasePassInterpolants,
	FVertexFactoryInterpolantsVSToPS Interpolants,
	VTPageTableResult LightmapVTPageTableResult,
	float3 VolumetricLightmapBrickTextureUVs,
	FMaterialPixelParameters MaterialParameters,
	float SceneDepth,
	float2 ScreenUV,
	FSubstratePixelHeader SubstratePixelHeader,
	FSubstrateData SubstrateData,
	inout float3 OutTransmittancePreCoverage,
	inout float OutCoverage
	)
{
	const uint PrimitiveId = MaterialParameters.PrimitiveId;
	const uint2 PixelPos = uint2(SvPosition.xy);
	const float Dither = InterleavedGradientNoise(PixelPos, View.StateFrameIndexMod8);
	const float3 AbsoluteWorldPosition = WSDemote(GetWorldPosition(MaterialParameters));
	const float3 TranslatedWorldPosition = MaterialParameters.WorldPosition_CamRelative;
	const float3 LightingPositionOffset = MaterialParameters.LightingPositionOffset;
	const half4 PrecomputedShadowFactors = GetPrimaryPrecomputedShadowMask(LightmapVTPageTableResult, Interpolants, MaterialParameters);
	float3 V = MaterialParameters.CameraVector;

	float3 Color = 0;
	OutCoverage = 0.0f;
	OutTransmittancePreCoverage = 1.0f;

	float SpecularScale = 1;
#if TRANSLUCENCY_ANY_VOLUMETRIC
	// No specular on volumetric translucency lighting modes
	SpecularScale = 0;
#endif

	const uint PrimitiveLightingChannelMask = GetPrimitive_LightingChannelMask(PrimitiveId);
	const float2 LocalPosition = SvPosition.xy - ResolvedView.ViewRectMin.xy;
	const uint GridIndex = ComputeLightGridCellIndex(uint2(LocalPosition.x, LocalPosition.y), SvPosition.w, EyeIndex);

	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		// Write Irradiance/AO data into header in order to be retrieve correctly during lighting
		#if SUBSTRATE_INLINE_SHADING
		HEADER_SETIRRADIANCE_AO(SubstratePixelHeader.State, SubstratePackIrradianceAndOcclusion(SubstratePixelHeader.IrradianceAO, 0));
		#endif

		// Update tree (coverage/transmittance/luminace weights)
		SubstratePixelHeader.SubstrateUpdateTree(SubstrateData, V, Settings, OutCoverage, OutTransmittancePreCoverage);

		//
		// Forward lighting
		//
		// 
		Substrate_for_unroll(int BSDFIdx = 0, BSDFIdx < SubstratePixelHeader.SubstrateTree.BSDFCount, ++BSDFIdx)
		{
		#define CurrentBSDF SubstratePixelHeader.SubstrateTree.BSDFs[BSDFIdx]

			if (SubstrateIsBSDFVisible(CurrentBSDF))
			{
				FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;	// Fake unused in SubstrateCreateBSDFContext when using Forward inline shading
				// Starting with environment lighting, the context does not specify a light
				FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, V);
					
				////
				//// Evaluate environment lighting
				////

				// Compute the bent normal from the BSDF normal
				const float Roughness = SubstrateGetBSDFRoughness(CurrentBSDF);
				const FShadingOcclusion ShadingOcclusion = ApplyBentNormal(MaterialParameters.CameraVector, SubstrateBSDFContext.N, GetWorldBentNormalZero(MaterialParameters), Roughness, SubstrateGetAO(SubstratePixelHeader));
				float IndirectOcclusion = 1.0f;
				half  IndirectIrradiance = 0.0f; 
				half3 WaterDiffuseIndirectLuminance = 0.f;

				// 1. Diffuse Sky evaluation using bent normal
				if (UseBasePassSkylightDiffuse > 0)
				{
					// Update a context specific for environment lighting
					FSubstrateBSDFContext EnvBSDFContext = SubstrateBSDFContext;
					EnvBSDFContext.N = ShadingOcclusion.BentNormal;
					EnvBSDFContext.SubstrateUpdateBSDFContext(EnvBSDFContext.L);

					// And evaluate diffuse parameters
					FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(EnvBSDFContext, false/*bEnableSpecular*/, Settings);

					// Evaluate diffuse lighting
					// Note: The evaluation is alway done for computing IndirectIrradiance, which is used for ImageBasedReflection evaluation
					{
						const float3 DiffuseNormal = SubstrateEnvLight.DiffuseNormal;
						const float3 DiffuseColorForIndirect = SubstrateEnvLight.DiffuseWeight;
						const float3 SubsurfaceColorForIndirect = SubstrateEnvLight.DiffuseBackFaceWeight;

						const bool bEvaluateBackface = any(SubsurfaceColorForIndirect > 0.0);

						const half NoL = max(0, dot(SubstrateBSDFContext.N, MobileDirectionalLight.DirectionalLightDirectionAndShadowTransition.xyz));
						const bool bApplySkyLighting = !(MATERIAL_TWOSIDED && LQ_TEXTURE_LIGHTMAP) || NoL == 0;

						half3 DiffuseIndirectLighting = 0.0f;
						half3 SubsurfaceIndirectLighting = 0.0f;
						GetPrecomputedIndirectLightingAndSkyLight(
							LightmapVTPageTableResult, 
							Interpolants, 
							bEvaluateBackface, 
							DiffuseNormal, 
							ResolvedView.SkyLightColor.rgb, 
							DiffuseIndirectLighting, 
							SubsurfaceIndirectLighting, 
							IndirectIrradiance, 
							WaterDiffuseIndirectLuminance);

						// SUBSTRATE_TODO: GetIndirectOcclusion require occlusion data, which are not available on mobile. The base pass has a 
						// AmbientOcclusionTexture which could be used, but it does not seems to be hooked for now
					#if 0 && FORWARD_SHADING && (MATERIALBLENDING_SOLID || MATERIALBLENDING_MASKED)
						float2 NearestResolvedDepthScreenUV = CalculateNearestResolvedDepthScreenUV(ScreenUV, MaterialParameters.ScreenPosition.w);

						IndirectOcclusion = GetIndirectOcclusion(NearestResolvedDepthScreenUV, SubstratePixelHeader.HasDynamicIndirectShadowCasterRepresentation());
						DiffuseIndirectLighting *= IndirectOcclusion;
						SubsurfaceIndirectLighting *= IndirectOcclusion;
						IndirectIrradiance *= IndirectOcclusion;
					#endif

						if (any((DiffuseColorForIndirect + SubsurfaceColorForIndirect) > 0.0f))
						{
							// For diffuse, we specify a perpendicular to the surface light direction for the transmittance to light to not be view dependent.
							const float DiffuseEnvLightingNoL = 1.0f;

							Color += (DiffuseIndirectLighting * DiffuseColorForIndirect + SubsurfaceIndirectLighting * SubsurfaceColorForIndirect) * 
									 LuminanceWeight(DiffuseEnvLightingNoL, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.DiffuseColor, ShadingOcclusion.DiffOcclusion);
						}
					}
				}

				// 2. Specular evaluation (using regular normal)
				#if (FORWARD_PER_PIXEL_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME)
				// Every components of forward shading reflection (Sky, SSR, Lumen) are enabled/disabled within GetImageBasedReflectionSpecular.
				{
					// Evaluate specular parameters without bent normal
					FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, true /*bEnableSpecular*/, Settings);

					// Evaluate specular lighting for the main lobe
					if (any(SubstrateEnvLight.SpecularWeight > 0.0f))
					{
						Color += GetImageBasedReflectionSpecular(SubstrateEnvLight.SpecularDirection, SubstrateEnvLight.SpecularSafeRoughness, MaterialParameters.WorldPosition_CamRelative, SubstrateBSDFContext.N, ShadingOcclusion.SpecOcclusion, IndirectIrradiance, GridIndex) *
								 SubstrateEnvLight.SpecularWeight * IndirectOcclusion * LuminanceWeight(SubstrateBSDFContext, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.SpecularColor, ShadingOcclusion.SpecOcclusion);
					}
					// And for the second lobe if needed
					if (any(SubstrateEnvLight.SpecularHazeWeight > 0.0f))
					{
						Color += GetImageBasedReflectionSpecular(SubstrateEnvLight.SpecularHazeDirection, SubstrateEnvLight.SpecularHazeSafeRoughness, MaterialParameters.WorldPosition_CamRelative, SubstrateBSDFContext.N, ShadingOcclusion.SpecOcclusion, IndirectIrradiance, GridIndex) *
								 SubstrateEnvLight.SpecularHazeWeight * IndirectOcclusion * LuminanceWeight(SubstrateBSDFContext, CurrentBSDF) * AOMultiBounce(SubstrateEnvLight.SpecularColor, ShadingOcclusion.SpecOcclusion);
					}
				}
				#endif


				////
				//// Evaluate translucent lighting volume / vertex lighting.
				////
				#define SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL    (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_DIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL || TRANSLUCENCY_LIGHTING_SURFACE_LIGHTINGVOLUME)
				#define SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL (TRANSLUCENCY_LIGHTING_VOLUMETRIC_PERVERTEX_NONDIRECTIONAL || TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL)

				#if SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL || SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL
				float4 VolumeLighting = 0;
				float3 InterpolatedLighting = 0;
				FSubstrateEvaluateResult BSDFEvaluate = SubstrateEvaluateBSDF(SubstrateBSDFContext, Settings);
				#endif

				////
				//// Evaluate emissive
				////
				Color += BSDF_GETEMISSIVE(CurrentBSDF) * CurrentBSDF.LuminanceWeightV;

				////
				//// Evaluate the single directional light selected for forward lighting.
				////
				const bool bHasDirectionalLight = any(MobileDirectionalLight.DirectionalLightColor.rgb > 0);
				BRANCH
				if (bHasDirectionalLight)
				{
					half DynamicShadowing = 1.0f;
					FDirectionalShadowSettings DirectionalShadowSettings = GetDefaultDirectionalShadowSettings();
					FDeferredLightData DirLightData = GetDirectionalLightData(MaterialParameters.ScreenPosition, DynamicShadowing, DirectionalShadowSettings);
					half4 DirLightAttenuation = half4(1, 1, DynamicShadowing, DynamicShadowing);
					uint DirLightingChannelMask = LIGHTING_CHANNEL_MASK;
					float3 ToDirLight = DirLightData.Direction;	// Already normalized
					float DirLightMask = 1;

				#if SUBSTRATE_TRANSLUCENT_MATERIAL
					if (DirLightData.bAffectsTranslucentLighting > 0)
				#endif
					{
						float DirectionalLightCloudShadow = 1.0f;
					#if NEEDS_BASEPASS_CLOUD_SHADOW_INTERPOLATOR
						DirectionalLightCloudShadow = BasePassInterpolants.VertexCloudShadow;
					#endif

						FTranslucencyLightingVector LightingVector = GetTranslucencyLightingVector(DirLightData, TranslatedWorldPosition);
					#if SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL
						GetVolumeLightingDirectional(LightingVector.AmbientLightingVector, LightingVector.DirectionalLightingVector, SubstrateBSDFContext.N, BSDFEvaluate.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), InterpolatedLighting, VolumeLighting);
					#elif SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL
						GetVolumeLightingNonDirectional(LightingVector.AmbientLightingVector, BSDFEvaluate.DiffuseColor, InterpolatedLighting, VolumeLighting);
					#else
						SubstrateBSDFContext.SubstrateUpdateBSDFContext(ToDirLight);

						FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
						Color += SubstrateForwardLightingCommon(
							Dither,
							Settings,
							DirLightData,
							ToDirLight,
							DirLightMask,
							DirLightAttenuation,
							InitRectTexture(),
							DirLightingChannelMask,
							PrimitiveLightingChannelMask,
							PrecomputedShadowFactors,
							TranslatedWorldPosition,
							SceneDepth,
							LuminanceWeight(SubstrateBSDFContext, CurrentBSDF),
							SubstratePixelHeader,
							SubstrateBSDFContext,
							BSDFEvaluate) * DirectionalLightCloudShadow;
					#endif
					}
				}

				////
				//// Evaluate local lights
				////
				#if ENABLE_CLUSTERED_LIGHTS && !DISABLE_FORWARD_LOCAL_LIGHTS && (FORWARD_SHADING || TRANSLUCENCY_LIGHTING_SURFACE_FORWARDSHADING) // Forbidden for Single Layer Water
				{
					// Basic implementation for FORWARD_LOCAL_LIGHTS where local lights are reloaded per BSDF layer. It is fast by default when layer==1.
					const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

					// Limit max to ForwardLightStruct.NumLocalLights.
					// This prevents GPU hangs when the PS tries to read from uninitialized NumCulledLightsGrid buffer
					const uint NumLightsInGridCell = min(CulledLightsGridHeader.NumLights, GetMaxLightsPerCell());

				#if SUBSTRATE_FORCE_SINGLE_LOCAL_LIGHT
					#define LOCAL_LIGHT_EARLY_OUT return Color;
					uint GridLightListIndex = 0;
					if (NumLightsInGridCell > 0)
				#else
					#define LOCAL_LIGHT_EARLY_OUT continue;
					LOOP
					for (uint GridLightListIndex = 0; GridLightListIndex < NumLightsInGridCell; GridLightListIndex++)
				#endif
					{
						const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, EyeIndex);
		
					#if MATERIALBLENDING_ANY_TRANSLUCENT && !SUBSTRATE_FORCE_SINGLE_LOCAL_LIGHT
						if(UnpackAffectsTranslucentLighting(LocalLight) == 0)
						{
							LOCAL_LIGHT_EARLY_OUT;
						}
					#endif

						half4 PreviewShadowMapChannelMask = 1;
						half4 DynamicShadowFactors = 1;
						uint  LocalLightChannelMask = LIGHTING_CHANNEL_MASK;
						FDeferredLightData LightData = ConvertToDeferredLight(LocalLight);
					
						// Rect light optionally supported in forward.
						// * This is not done explicitely in the legacy path (resulting in no lighting) but we do it for Substrate in order to avoid visual artefact.
						// * This can be removed when LocalLight properly supports rect lights.
						if (LightData.bRectLight && !SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT)
						{
							LOCAL_LIGHT_EARLY_OUT;
						}

						// When disabled, force the compiler to remove the texture atlas sampling to avoid taking an extra texture sampler
						#if !SUPPORT_RECTLIGHT_ON_FORWARD_LIT_TRANSLUCENT
						LightData.bRectLight = false;
						#endif
						const FRectTexture LocalRectTexture = ConvertToRectTexture(LightData);

						float DynamicShadowing = dot(PreviewShadowMapChannelMask, DynamicShadowFactors);
						float4 LightAttenuation = float4(1, 1, DynamicShadowing.x, DynamicShadowing.x);

						float3 L = LightData.Direction;	// Already normalized
						float3 ToLight = L;
						float LightMask = 1;
						if (LightData.bRadialLight)
						{
							LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);
						}

						if (!IsLightVisible(LocalLight, TranslatedWorldPosition))
						{
							LOCAL_LIGHT_EARLY_OUT;
						}

						FTranslucencyLightingVector LightingVector = GetTranslucencyLightingVector(LightData, TranslatedWorldPosition);
					#if SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_DIRECTIONAL
						GetVolumeLightingDirectional(LightingVector.AmbientLightingVector, LightingVector.DirectionalLightingVector, SubstrateBSDFContext.N, BSDFEvaluate.DiffuseColor, GetMaterialTranslucencyDirectionalLightingIntensity(), InterpolatedLighting, VolumeLighting);
					#elif SUBSTRATE_TRANSLUCENCY_LIGHTING_VOLUMETRIC_NONDIRECTIONAL
						GetVolumeLightingNonDirectional(LightingVector.AmbientLightingVector, BSDFEvaluate.DiffuseColor, InterpolatedLighting, VolumeLighting);
					#else
						// Update the Substrate BSDF context accordgin to the new L
						SubstrateBSDFContext.SubstrateUpdateBSDFContext(ToLight);

						FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
						Color += SubstrateForwardLightingCommon(
							Dither,
							Settings,
							LightData,
							ToLight,
							LightMask,
							LightAttenuation,
							LocalRectTexture,
							LocalLightChannelMask,
							PrimitiveLightingChannelMask,
							PrecomputedShadowFactors,
							TranslatedWorldPosition,
							SceneDepth,
							LuminanceWeight(SubstrateBSDFContext, CurrentBSDF),
							SubstratePixelHeader,
							SubstrateBSDFContext,
							BSDFEvaluate);
					#endif
					}
					#undef LOCAL_LIGHT_EARLY_OUT
				}
				#endif // !DISABLE_FORWARD_LOCAL_LIGHTS

			}// SubstrateIsBSDFVisible
		#undef CurrentBSDF
		}
	}

	return Color;
}


