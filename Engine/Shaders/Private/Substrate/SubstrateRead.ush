// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Sanity guard.  
#ifndef SUBSTRATE_ENABLED
#define SUBSTRATE_ENABLED 1
#error SUBSTRATE_ENABLED needs to be defined
#endif

#if SUBSTRATE_ENABLED

#include "../Common.ush"
#include "Substrate.ush"
#include "/Engine/Private/DeferredShadingCommon.ush"

struct FSubstrateGBufferBSDF
{
	FSubstrateBSDF BSDF;
	float3x3 BSDFTangentBasis;
	float BSDFAO;
};

FSubstrateGBufferBSDF SubstrateReadGBufferBSDF(in FScreenSpaceData ScreenSpaceData)
{
	const FGBufferData GBuffer = ScreenSpaceData.GBuffer;

	FSubstrateBSDF BSDF = (FSubstrateBSDF)0;
	float3x3 BSDFTangentBasis = (float3x3)0;
	float BSDFAO = 0.0f;

#if SUBSTRATE_COMPLEXPATH
	if (GBuffer.Anisotropy != 0)
	{
		BSDFTangentBasis[0] = GBuffer.WorldTangent;
		BSDFTangentBasis[1] = cross(GBuffer.WorldNormal, GBuffer.WorldTangent);
		BSDFTangentBasis[2] = GBuffer.WorldNormal;
	}
	else
#endif
	{
		BSDFTangentBasis = GetTangentBasis(GBuffer.WorldNormal);
	}

	BSDFAO = ScreenSpaceData.AmbientOcclusion;

	{
		FSubstratePixelFootprint Footprint = SubstrateGetPixelFootprint(1.0f, 1.0f, GBuffer.Curvature);

		const float3 Emissive = 0.0f;
		const float  SpecularProfileId = 0.0f;

		const float  GlintValue = 1.0f;
		const float2 GlintUV = 0.0f;
		const float2 GlintUVdx = 0.0f;
		const float2 GlintUVdy = 0.0f;

		float3		SSSMFP = 0.0f;
		float		SSSMFPScale = 1.0f;
		float		SSSPhaseAnisotropy = 0.0f;

		float		Roughness = GBuffer.Roughness;
		float		SecondRoughness = 0.0f;
		float		SecondRoughnessWeight = 0.0f;
		float		SecondRoughnessAsSimpleClearCoat = 0.0f;

		float		FuzzAmount = 0.0f;
		float3		FuzzColor = 0.0f;
		float		FuzzRoughness = 0.0f;

		uint		SSSProfileID = SSS_PROFILE_ID_INVALID;
		uint		SSSType = SSS_TYPE_NONE;
		float		ThicknessCm = 1.0f;
		bool		bIsThinSurface = false;

		// Directly use DiffuseColor and SpecularColor (instead of ComputeDiffuseAlbedo() / ComputeF0() with BaseColor, Metallic, Specular), 
		// as they contain Diffuse/Specular override which are used for view modes.
		float3 DiffuseAlbedo = GBuffer.DiffuseColor;
		float3 F0 = GBuffer.SpecularColor;
		float3 F90 = 1.0f;

		float ClearCoatUseSecondNormal = 0.0f;
		float3 ClearCoatBottomNormal = GBuffer.WorldNormal;

		bool   bIsAtTheBottomOfTopology = true;
		uint   SharedLocalBasisIndex = 0;
		uint   SharedLocalBasisTypes = 0;

	#if SUBSTRATE_COMPLEXPATH
		if (GBuffer.ShadingModelID == SHADINGMODELID_HAIR)
		{
			const float Scatter = GBuffer.Metallic;
			const float BackLit = GBuffer.CustomData.b;
			const bool bComplexTransmittance = ShouldUseHairComplexTransmittance(GBuffer);
			BSDF = GetSubstrateHairBSDF(GBuffer.BaseColor, Scatter, GBuffer.Specular, Roughness, BackLit, Emissive, bComplexTransmittance, SharedLocalBasisIndex).InlinedBSDF;

		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_EYE)
		{
			SSSProfileID = ExtractSubsurfaceProfileInt(GBuffer);
			const float IrisMask = 1.0f - GBuffer.CustomData.w;
			float IrisDistance = 0.0;
			float3 IrisNormal = GBuffer.WorldNormal;
			float3 IrisPlaneNormal = GBuffer.WorldNormal;

		#if IRIS_NORMAL
			const float2 IrisNormalDelta = float2(GBuffer.CustomData.y, GBuffer.CustomData.z) * 2 - (256.0 / 255.0);
			const float2 WorldNormalOct = UnitVectorToOctahedron(GBuffer.WorldNormal);
			IrisNormal = OctahedronToUnitVector(WorldNormalOct + IrisNormalDelta);
			IrisDistance = 1.0; // Set at 1 in order to let's the mask drive the interpolation. See SubstrateEvaluation.ush.
		#else
			IrisNormal = OctahedronToUnitVector(GBuffer.CustomData.yz * 2 - 1);
			IrisPlaneNormal = IrisNormal;
			IrisDistance = GBuffer.StoredMetallic;
		#endif
			BSDF = GetSubstrateEyeBSDF(DiffuseAlbedo, F0, Roughness, IrisMask, IrisDistance, IrisNormal, IrisPlaneNormal, SSSProfileID, Emissive, SharedLocalBasisIndex).InlinedBSDF;
		}
		else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
		{
			// special read from custom data
			float2 PatternUVs = 0.0f;
			float ToonProfileId = 0.0f;
			UnpackToonCustomData(PackRGBA8(GBuffer.CustomData), PatternUVs, ToonProfileId);

			BSDF = GetSubstrateToonBSDF(
				ToonProfileId,
				GBuffer.BaseColor, GBuffer.Metallic, GBuffer.Specular, Roughness,
				Emissive,
				PatternUVs,
				GBuffer.Anisotropy, GBuffer.WorldTangent,
				SharedLocalBasisIndex, SharedLocalBasisTypes).InlinedBSDF;
		}
		else
	#endif
		{
		#if SUBSTRATE_SINGLEPATH || SUBSTRATE_COMPLEXPATH
			if (GBuffer.ShadingModelID == SHADINGMODELID_CLOTH)
			{
				FuzzColor = ExtractSubsurfaceColor(GBuffer);
				FuzzAmount = GBuffer.CustomData.a;
				FuzzRoughness = 0.5f; // Fuzz roughness is not stored into the gbuffer. Using GBuffer.Roughness it not ideal as low value, makes most of the fuzz contribution to vanish. Instead hardcode value.
			}
			else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE || GBuffer.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
			{
				SSSType = SSS_TYPE_DIFFUSION;	//  SSS_TYPE_DIFFUSION_PROFILE will be selected in InternalGetSubstrateSlabBSDF based on SSSProfileID and SSSMFPScale.
				SSSMFP = 0;
				SSSProfileID = ExtractSubsurfaceProfileInt(GBuffer);
				SSSMFPScale = GBuffer.CustomData.a;
				SSSPhaseAnisotropy = 0.93f;

				if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE)
				{
				#if SUBSTRATE_GBUFFER_FORMAT == 0 && !FORWARD_SHADING // To match legacy GetProfileDualSpecular
					// Evaluate dual specular roughnesses into secondary SecondRoughness Slab parameters.
					GetSubsurfaceProfileDualSpecular(SSSProfileID, Roughness, SSSMFPScale, Roughness, SecondRoughness, SecondRoughnessWeight);
					SecondRoughnessAsSimpleClearCoat = 0.0f; 
				#endif
				}
			}
			else if (GBuffer.ShadingModelID == SHADINGMODELID_SUBSURFACE)
			{
				SSSType = SSS_TYPE_WRAP;
				{
					// See SubstrateGetBSDFSubSurfaceColor
					float3 Extinction = TransmittanceToExtinction(ExtractSubsurfaceColor(GBuffer), SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
					SSSMFP = (1.0f / max(SUBSTRATE_EPSILON, Extinction));
				}
				SSSPhaseAnisotropy = 1.f - GBuffer.CustomData.a;
				SSSMFPScale = 1.0f;
			}
			else if (GBuffer.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
			{
				SSSType = SSS_TYPE_TWO_SIDED_WRAP;
				{
					// See SubstrateGetBSDFSubSurfaceColor
					float3 Extinction = TransmittanceToExtinction(ExtractSubsurfaceColor(GBuffer), SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM * CENTIMETER_TO_METER);
					SSSMFP = (1.0f / max(SUBSTRATE_EPSILON, Extinction));
				}
				SSSPhaseAnisotropy = 1.f - GBuffer.CustomData.a;
				SSSMFPScale = 1.0f;
			}
			else if (GBuffer.ShadingModelID == SHADINGMODELID_CLEAR_COAT)
			{
			#if SUBSTRATE_FASTPATH==0 && CLEAR_COAT_BOTTOM_NORMAL 
				ClearCoatUseSecondNormal = 1.0f;
				const float2 oct1 = ((float2(GBuffer.CustomData.a, GBuffer.CustomData.z) * 4) - (512.0 / 255.0)) + UnitVectorToOctahedron(GBuffer.WorldNormal);
				ClearCoatBottomNormal = OctahedronToUnitVector(oct1);
			#endif

				SecondRoughnessWeight = GBuffer.CustomData.x;
				SecondRoughness = GBuffer.CustomData.y;
				SecondRoughnessAsSimpleClearCoat = (SecondRoughnessWeight > 0.0f || any(GBuffer.WorldNormal != ClearCoatBottomNormal)) ? 1.0f : 0.0f;
			}
		#endif

			BSDF = InternalGetSubstrateSlabBSDF(
				Footprint,
				GBuffer.WorldNormal,
				DiffuseAlbedo, F0, F90,
				Roughness, GBuffer.Anisotropy,
				SSSProfileID, true /*bSupportDefaultSSSProfile*/, SSSMFP, SSSMFPScale, SSSPhaseAnisotropy, SSSType,
				Emissive,
				SecondRoughness, SecondRoughnessWeight, SecondRoughnessAsSimpleClearCoat,
				ClearCoatUseSecondNormal, ClearCoatBottomNormal,
				FuzzAmount, FuzzColor, FuzzRoughness,
				GlintValue, GlintUV, GlintUVdx, GlintUVdy,
				SpecularProfileId,
				ThicknessCm,
				bIsThinSurface,
				bIsAtTheBottomOfTopology,
				SharedLocalBasisIndex, SharedLocalBasisTypes,
				true /*bSupportSSSDiffusion*/).InlinedBSDF;
			
			SLAB_SSSMFP(BSDF) = SanitizeSSSMFP(SLAB_SSSMFP(BSDF));
		}

		#if SUBSTRATE_INLINE_SHADING && !RAYTRACINGSHADER
		BSDF.Coverage = 1;
		BSDF.Emissive = 0;
		BSDF.ThicknessCm = (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP) ? SUBSTRATE_SIMPLEVOLUME_THICKNESS_CM : SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM; // Identity rescaling in InternalGetSubstrateSlabBSDF 
		BSDF.TmpMFP = SSSMFP;
		BSDF.TopLayerDataWeight = 1;
		#endif

		BSDF.OperatorIndex = 0;
		BSDF.LuminanceWeightV = 1.0f;
		BSDF.CoverageAboveAlongN = 0.0f;
		BSDF.TransmittanceAboveAlongN = 1.0f;
		BSDF.bIsBottom = true;
		BSDF.bIsTop = true;
		BSDF_SETISTOPLAYER(BSDF, 1);
	}

	FSubstrateGBufferBSDF Out;
	Out.BSDF = BSDF;
	Out.BSDFTangentBasis = BSDFTangentBasis;
	Out.BSDFAO = BSDFAO;
	return Out;
}

FSubstrateBSDF UnpackSubstrateBSDFIn(inout FSubstrateAddressing SubstrateAddressing)
{
	FSubstrateGBufferBSDF GBufferBSDF = (FSubstrateGBufferBSDF)0; 
#if SHADING_PATH_DEFERRED
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceDataUint(SubstrateAddressing.PixelCoords);
#else
	const float2 UV = (float2(SubstrateAddressing.PixelCoords) + 0.5f) * View.BufferSizeAndInvSize.zw;
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(UV);
#endif
	GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
#endif
	return GBufferBSDF.BSDF;
}

#endif // SUBSTRATE_ENABLED
