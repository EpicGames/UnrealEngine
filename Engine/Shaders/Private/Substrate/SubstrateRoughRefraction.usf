// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"



// Gaussian function with a=1/(c*sqrt(2*PI)) so it represents the probability density function of a normally distributed random variable with expected value mu = b for some variance and integrale = 1;
float GaussianFunction(float x, float Mean, float Variance)
{
	// To avoid numerical precision issue, clamp the variance (stddev =0.001)
	Variance = max(0.000001f, Variance);

	float StandardDeviation = sqrt(Variance);
	float FactorA = 1 / (StandardDeviation * sqrt(2 * PI));
	float Bell = exp(-abs((x - Mean) * (x - Mean)) / (2 * Variance));
	return FactorA * Bell;
}
float GaussianFunction(float2 P, float2 Mean, float Variance)
{
	// To avoid numerical precision issue, clamp the variance (stddev =0.001)
	Variance = max(0.000001f, Variance);

	float StandardDeviation = sqrt(Variance);
	float FactorA = 1 / (StandardDeviation * sqrt(2 * PI));
	float Bell = exp(-((P.x - Mean.x) * (P.x - Mean.x)) / (2 * Variance) - ((P.y - Mean.y) * (P.y - Mean.y)) / (2 * Variance));
	return FactorA * Bell;
}



#if OPAQUE_ROUGH_REFRACTION_PS

#include "/Engine/Private/Substrate/Substrate.ush"

Texture2D<float4> SeparatedOpaqueRoughRefractionSceneColor;
float2 BlurDirection;
float BlurScale;

void OpaqueRoughRefractionPS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 PixelPos = SVPos.xy;
	float2 UvBuffer = PixelPos * View.BufferSizeAndInvSize.zw;

	// First read unblurred layers at the bottom of the top layer
	OutColor = SeparatedOpaqueRoughRefractionSceneColor[uint2(PixelPos)];

#if PERMUTATION_ENABLE_BLUR && SUBSTRATE_OPAQUE_ROUGH_REFRACTION_ENABLED
	FSubstrateOpaqueRoughRefractionData ORRData = SubstrateUnpackOpaqueRoughRefractionData(Substrate.OpaqueRoughRefractionTexture[PixelPos]);
	ORRData.VarianceCm *= BlurScale;

	const float DeviceZ = LookupDeviceZ(UvBuffer);
	const float SceneDepthCm = ConvertFromDeviceZ(DeviceZ);

	const float2 PixelRadiusCm2 = SceneDepthCm * GetTanHalfFieldOfView() * View.ViewSizeAndInvSize.zw;
	const float PixelRadiusCm = min(PixelRadiusCm2.x, PixelRadiusCm2.y);

	const float StandardDeviationCm = sqrt(ORRData.VarianceCm);
	const float StandardDeviationPixelSpace = StandardDeviationCm / PixelRadiusCm;	// Be careful to do that conversion on linear standard deviation, not variance
	const float VariancePixelSpace = StandardDeviationPixelSpace * StandardDeviationPixelSpace;

	const float PerceptualBlurAcceptanceFactor = 2.5f;
	const bool OpaqueRoughRefractionEnabled = ORRData.OpaqueRoughRefractionEnabled;// && (PixelRadiusCm < PerceptualBlurAcceptanceFactor* ORRData.VarianceCm);

	// We are now going to blur the bottom layer luminance according to the top layer variance resulting from the roughness.
	// For this we have: measure GGX variance for a perpendicaular view, eta=1.33 and varying roughness. At runtim that variance is approximated by a Gaussian.

	if (OpaqueRoughRefractionEnabled)
	{
		// Proceed blurring the bottom layers if needed according to the blur footprint.
		float3 WeightedLuminance = 0.0f;
		float AccumulatedWeights = 0.0f;

		// A area of radius StandardDeviation*3 around the mean of a guassian normal distribution (=mu) encompass 99% of the energy.
		const float RadiusAroundCenterMu = 3.0f;
		const float BlurStep = 0.5f;

		// Randomly offset the blur center position around according to step size to leverage TAA and smooth the blur.
		const float BlurRandomness = InterleavedGradientNoise(PixelPos, float(View.StateFrameIndexMod8)) - 0.5; // In [-0.5, 0.5]
		const float RandAngle = BlurRandomness * 2.0f * PI;
		float CosA, SinA;
		sincos(RandAngle, SinA, CosA);
		const float2 BlurOrigin = PixelPos + BlurStep * StandardDeviationPixelSpace * float2(CosA, SinA);

		const float4 UvViewRectMinMax = float4(float2(View.ViewRectMinAndSize.xy) + 0.5f, float2(View.ViewRectMinAndSize.xy + View.BufferSizeAndInvSize.xy) - 0.5f) * View.BufferSizeAndInvSize.zwzw;

		for (float u = -RadiusAroundCenterMu; u <= RadiusAroundCenterMu; u += BlurStep) // steps of 0.5 means 13 taps
		{
			const float OffsetPixelSpace = u * StandardDeviationPixelSpace;
			const float2 SampleUV = (BlurOrigin + BlurDirection * OffsetPixelSpace) * View.BufferSizeAndInvSize.zw; 
			const uint2 SamplePixelPos = uint2(SampleUV * View.BufferSizeAndInvSize.xy);

			float Weight = GaussianFunction(OffsetPixelSpace, 0.0f, VariancePixelSpace);

			// Do not accept rough refraction if the SampleUV is out of the view rect
			const bool bSampleIsWithinViewRect = all(SampleUV >= UvViewRectMinMax.xy) && all(SampleUV <= UvViewRectMinMax.zw);
			Weight *= bSampleIsWithinViewRect ? 1.0f : 0.0f;

			// Do not accept rough refraction if the sampled material does not do rough refraction. Solved well by depth difference below so disabled.
			//FSubstrateOpaqueRoughRefractionData ORRSampleData = SubstrateUnpackOpaqueRoughRefractionData(Substrate.OpaqueRoughRefractionTexture[SamplePixelPos]);
			//Weight *= ORRSampleData.OpaqueRoughRefractionEnabled ? 1.0f : 0.0f;

			// Do not accept rough refraction if the depth difference is too high. Solves a lot of problems such as: 
			//  - materials/sky being further away leaking,
			//  - two materials hacing rough refraction touching on screen with a too large depth difference.
			// Using the sample UV with bilinear interpolation for such an exclusion mechanism is not perfect but does work well in most cases.
			// The true solution would be to only take the average of the 4 neightboor samples according to bilinear weights modulated by the exclusion filter itself.
			const float DepthDifferenceThresholdCm = 5.0f;
			const float SampleSceneDepthCm = ConvertFromDeviceZ(LookupDeviceZ(SampleUV));
			Weight *= abs(SampleSceneDepthCm - SceneDepthCm) < DepthDifferenceThresholdCm ? 1.0f : 0.0f;

			// Do not accepth rough refraction if the normal difference is too high: 
			// => this could prove complex to make work in many cases with high frequency normalmapped surfaces. A geometry normal would be better to use.

			WeightedLuminance += SeparatedOpaqueRoughRefractionSceneColor.SampleLevel(View.SharedBilinearClampedSampler, SampleUV, 0).rgb * Weight;
			AccumulatedWeights += Weight;
		}

		const float bIsBlurValid = AccumulatedWeights > 0.0;
		const float3 BlurredSceneColor = bIsBlurValid ? WeightedLuminance / AccumulatedWeights : 0.0f;

		// Combine blurred layers below with unblurred according to coverage
		OutColor = float4(lerp(OutColor.rgb, BlurredSceneColor, bIsBlurValid ? ORRData.Coverage : 0.0), OutColor.a);
		//OutColor *= float4(0, 1, 0, 1);
	}
#endif

}

#endif



#ifdef SUBSTRATE_RND_SHADERS

//////////////////////////////////////////////////////////////////////////
// RnD shaders only used when enabled locally
//////////////////////////////////////////////////////////////////////////

#include "/Engine/Private/MiniFontCommon.ush"
#include "/Engine/Private/ShaderPrint.ush"
#include "/Engine/Private/ColorMap.ush"
#include "/Engine/Private/BRDF.ush"
#include "/Engine/Private/MonteCarlo.ush"

#include "/Engine/Private/PathTracing/Material/PathTracingFresnel.ush"
#include "/Engine/Private/PathTracing/Material/PathTracingGlossy.ush"

#include "/Engine/Private/Substrate/SubstrateStatisticalOperators.ush"

#define MODE_REFLECTION 0

// Updated from http://jcgt.org/published/0007/03/04/
bool Slabs(float3 P0, float3 P1, float3 RayOrigin, float3 InvRaydir, out float OutTMin, out float OutTMax)
{
	float3 T0 = (P0 - RayOrigin) * InvRaydir;
	float3 T1 = (P1 - RayOrigin) * InvRaydir;
	float3 TMin = min(T0, T1);
	float3 TMax = max(T0, T1);
	float MaxTMin = max(max(TMin.x, TMin.y), TMin.z);
	float MinTMax = min(min(TMax.x, TMax.y), TMax.z);
	OutTMin = MaxTMin;
	OutTMax = MinTMax;
	return MaxTMin <= MinTMax;
}

float GetEta12()
{
	const float AirIOR = 1.0f;
	const float WaterIOR = 1.33f;
	const float EtaIn = WaterIOR / AirIOR;	// We always consider entering from air to water

	return EtaIn;
}

#define WOLRD_NORMAL float3(0, 0, 1)

// returns true the direction is valid for the current mode (refraction or reflection)
bool GetGGXDirection(float2 RandomUV, float Roughness, float3 V, inout float3 L, inout float Value, float Eta12)
{ 
	float2 Alpha; 
	GetGGXBasis(Roughness, WOLRD_NORMAL, Alpha);

	float4 H = ImportanceSampleVisibleGGX(RandomUV, Alpha, V, false);

	Value = 0;
	L = 0;
	float F = 0;
	const float RandSample = MODE_REFLECTION ? -0.01f : 1.0f;
	const bool bIsRefraction = SampleRefraction(-V, H.xyz, Eta12, RandSample, L, F);
	const bool bTotalInternalReflection = F == 1.0f;
	if (bIsRefraction && !bTotalInternalReflection)
	{
		Value = GGXEvalRefraction(L, V, H.xyz, Alpha, Eta12).x;
		return MODE_REFLECTION == 0;
	}

	// Reflection
	Value = GGXEvalReflection(L, V, H.xyz, Alpha, false).x;
	return MODE_REFLECTION == 1;
}



struct FLobeStat
{
	float Count;
	float2 Mean;
	float2 M2;
	float2 Variance;
	float pad;
};

// See Welford's algorithm https://en.wikipedia.org/wiki/Algorithms_for_calculating_variance
void LobStatUpdate(inout FLobeStat LobeStat, in float2 InPos)
{
	LobeStat.Count += 1;
	float2 Delta = InPos - LobeStat.Mean;
	LobeStat.Mean += Delta / LobeStat.Count;
	float2 Delta2 = InPos - LobeStat.Mean;
	LobeStat.M2 += Delta * Delta2;
}

void LobStatFinalize(inout FLobeStat LobeStat)
{
	if (LobeStat.Count < 2)
	{
		// nan
	}
	else
	{
		// Mean is already valid
		LobeStat.Variance = LobeStat.M2 / LobeStat.Count;
		//LobeStat.SampleVariance = LobeStat.M2 / (LobeStat.Count - 1);
	}
}



#if defined(EVALUATE_ROUGH_REFRACTION_LOBE_CS) || defined(VISUALIZE_ROUGH_REFRACTION_PS)

struct FUI
{
	float IncidentAngle;
	float SlabThickness;
	float Roughness;
	float Eta12;

	// Derived
	float a2;
	float3 V;
};

FShaderPrintContext GetUIContext(bool bActive)
{
	return InitShaderPrintContext(bActive, uint2(100, 100));
}

FUI GetUI(inout FShaderPrintContext Context)
{
	FUI UI;

	UI.IncidentAngle = AddSlider(Context, TEXT("IncidentAngle"), 0.0f, GetDefaultFontColor(), 0.0f, 0.5f * PI);

	Newline(Context);

	Newline(Context);

	UI.Roughness = AddSlider(Context, TEXT("Roughness"), 0.5f, GetDefaultFontColor(), 0.0f, 1.0f);

	Newline(Context);

	Newline(Context);

	UI.Eta12 = AddSlider(Context, TEXT("Eta12"), 1.3f, GetDefaultFontColor(), 0.5f, 1.5f);

	Newline(Context);

	Newline(Context);

	UI.SlabThickness = AddSlider(Context, TEXT("Slab Thickness"), 5.0f, GetDefaultFontColor(), 0.0f, 50.0f);

	Newline(Context);

	UI.a2 = Pow4(UI.Roughness);
	UI.V = float3(sin(UI.IncidentAngle), 0, cos(UI.IncidentAngle));

	return UI;
}

void DrawCircleXY(inout FShaderPrintContext Context, float3 Center, float2 Radius, float4 ColorAlpha)
{
	const float TStep = 0.1f;
	for (float t = 0; t <= 1.0; t += TStep)
	{
		float S0, C0;
		float S1, C1;
		sincos(t * 2 * PI, S0, C0);
		sincos((t + TStep) * 2 * PI, S1, C1);
		float3 P0 = Center + float3(C0, S0, 0.0) * float3(Radius, 0.0);
		float3 P1 = Center + float3(C1, S1, 0.0) * float3(Radius, 0.0);

		AddLineWS(Context, P0, P1, ColorAlpha);
	}
}

#endif



#ifdef EVALUATE_ROUGH_REFRACTION_LOBE_CS

uint TraceSqrtSampleCount;

RWTexture2D<uint> SampleCountTextureUAV;

RWStructuredBuffer<FLobeStat> LobStatisticsBufferUAV;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void EvaluateRoughRefractionLobeCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (all(ThreadId == 0))
	{
		FShaderPrintContext Context = GetUIContext(true);
		FUI UI = GetUI(Context);

		FLobeStat LobeState = (FLobeStat)0;

		float Count = 0;
		float2 Mean = 0;
		float2 Variance = 0;

		for (float u = 0.5f; u < float(TraceSqrtSampleCount); u++)
		{
			for (float v = 0.5f; v < float(TraceSqrtSampleCount); v++)
			{
				float2 RandomUV = float2(u, v) / float(TraceSqrtSampleCount);
				float3 L = 0;
				float Value = 0;
				if (GetGGXDirection(RandomUV, UI.Roughness, UI.V, L, Value, UI.Eta12) && Value > 0.0f)
				{
					float2 SampleUV = L.xy * 0.5 + 0.5;
					SampleCountTextureUAV[uint2(SampleUV * 64.0f)] += 1;

					const float3 RayOnPlane = L.xyz / abs(L.z);
					LobStatUpdate(LobeState, RayOnPlane.xy);

					Count++;
					Mean += RayOnPlane.xy;
				}
			}
		}

/*		Mean = Mean / Count;
		Count = 0;
		for (float u = 0.5f; u < float(TraceSqrtSampleCount); u++)
		{
			for (float v = 0.5f; v < float(TraceSqrtSampleCount); v++)
			{
				float2 RandomUV = float2(u, v) / float(TraceSqrtSampleCount);
				float3 L = 0;
				float3 Value = 0;
				if (GetGGXDirection(RandomUV, UI.Roughness, UI.V, L, Value, UI.Eta12) && Value > 0.0f)
				{
					float2 SampleUV = L.xy * 0.5 + 0.5;
					SampleCountTextureUAV[uint2(SampleUV * 64.0f)] += 1;

					LobStatUpdate(LobeState, L.xy);

					Count++;
					Variance += (L.xy - Mean) * (L.xy - Mean);
				}
			}
		}
		Variance = Variance / Count;
		LobeState.Mean = Mean;
		LobeState.Variance = Variance;*/

		LobStatFinalize(LobeState);
		LobStatisticsBufferUAV[0] = LobeState;
	}
}

#endif // EVALUATE_ROUGH_REFRACTION_LOBE_CS



#ifdef EVALUATE_ROUGH_REFRACTION_DATA_CS

uint TraceSqrtSampleCount;

RWBuffer<float2> RoughRefracDataUAV;

// Shader used to export variance based on roughness for curve fitting
[numthreads(THREADGROUP_SIZE, 1, 1)]
void RoughRefracDataCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const float Roughness = float(ThreadId.x) / float(THREADGROUP_SIZE);

	FLobeStat LobeState = (FLobeStat)0;

	for (float u = 0.5f; u < float(TraceSqrtSampleCount); u++)
	{
		for (float v = 0.5f; v < float(TraceSqrtSampleCount); v++)
		{
			float2 RandomUV = float2(u, v) / float(TraceSqrtSampleCount);
			float3 V = float3(0, 0, 1);	// Only considering perpendicular view for now
			float Eta12 = 1.33;			// Only considering air to water for now

			float3 L = 0;
			float Value = 0;
			if (GetGGXDirection(RandomUV, Roughness, V, L, Value, Eta12) && Value > 0.0f)
			{
				// Project onto the plane at distance 1 to consider the blur resulting at a distance of 1.
				const float3 RayOnPlane = L.xyz / abs(L.z);
				LobStatUpdate(LobeState, RayOnPlane.xy);
			}
		}
	}

	LobStatFinalize(LobeState);

	RoughRefracDataUAV[ThreadId.x] = float2(Roughness, dot(LobeState.Variance, float2(0.5, 0.5)));	// Roughness => Mean of both X and Y variances
}

#endif // EVALUATE_ROUGH_REFRACTION_DATA_CS



#ifdef VISUALIZE_ROUGH_REFRACTION_PS

float TraceDomainSize;
uint SlabInterfaceLineCount;

Texture2D<uint> SampleCountTexture;

StructuredBuffer<FLobeStat> LobStatisticsBuffer;

FSubstrateLobeStatistic SubstrateGetRefractedLobe2(FSubstrateLobeStatistic WiLobe, float3 InterfaceFDG, float InterfaceRoughness, float InterfaceEta12)
{
	FSubstrateLobeStatistic WoLobe;

	WoLobe.E = WiLobe.E * (1.0f - InterfaceFDG);

	// The lobe is on the oposite side of the surface, and the direction account for change of medium IOR.
	WoLobe.Mu.xy = -WiLobe.Mu.xy * InterfaceEta12;
	WoLobe.Mu.z = -sign(WiLobe.Mu.z) * sqrt(1.0 - dot(WoLobe.Mu.xy, WoLobe.Mu.xy)); // derive z from vector projected xy

	// Equation 10, we have also noticed variance explosion at grazing angle so we make the inner product be 1 for now.
	const float OmegaIDotN = 1.0f; //WiLobe.Mu.z;
	const float OmegaTDotN = 1.0f; //WoLobe.Mu.z;
	//const float S = 0.5f * (1.0f + InterfaceEta12 * (abs(OmegaIDotN) / abs(OmegaTDotN)));					// equation 10 from paper
	const float S = 0.5f * abs((OmegaTDotN * InterfaceEta12 - OmegaIDotN) / (OmegaTDotN * InterfaceEta12));	// equation from paper source code

	WoLobe.Sigma = (WiLobe.Sigma / InterfaceEta12) + SubstrateLobeRoughnessToVariance(InterfaceRoughness * S);

	return WoLobe;
}

void VisualizeRoughRefractionPS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	OutColor = float4(0.0, 0.0f, 0.0, 0.5f);

	float2 PixelPos = SVPos.xy;
	float2 UvBuffer = PixelPos * View.BufferSizeAndInvSize.zw;
	float3 WorldDir = GetScreenWorldDir(SVPos);
	float3 CamWorldPos = DFDemote(PrimaryView.WorldCameraOrigin);

	FLobeStat LobeState = LobStatisticsBuffer[0];

	const float CountScale = 0.01f;
	
	FShaderPrintContext Context = GetUIContext(all(uint2(PixelPos.xy) == uint2(10, 10)));
	FUI UI = GetUI(Context);

	float3 SlabEdgeX = float3(60.0f, 0.0f, 0.0f);
	float3 SlabEdgeY = float3(0.0f, 60.0f, 0.0f);
	float3 SlabEdgeZ = float3(0.0f, 0.0f, UI.SlabThickness);
	float3 SlabCenter = CamWorldPos + View.ViewForward * 35.0f;
	float3 RefractionP= SlabCenter + 0.5 * SlabEdgeZ;
	const float SlabTopZ    = SlabCenter.z  + 0.5 * UI.SlabThickness;
	const float SlabBottomZ = SlabCenter.z - 0.5 * UI.SlabThickness;

	const float VisProjectedRayRadius = 15.0f;

	AddLineWS(Context, RefractionP, RefractionP + WOLRD_NORMAL * 25.0, float4(0, 0, 1, 1));
	AddLineWS(Context, RefractionP, RefractionP + UI.V * 25.0, float4(0, 1, 1, 1));

	float SlabTMin = -1.0f;
	float SlabTMax = -1.0f;
	float3 SlabP0 = SlabCenter - 0.5 * (SlabEdgeX + SlabEdgeY + SlabEdgeZ);
	float3 SlabP1 = SlabCenter + 0.5 * (SlabEdgeX + SlabEdgeY + SlabEdgeZ);
	if(Slabs(SlabP0, SlabP1, CamWorldPos, 1.0f / WorldDir, SlabTMin, SlabTMax))
	{
		float3 P = CamWorldPos + WorldDir * SlabTMin;
		if (P.z > (SlabBottomZ + 0.01))
		{
			P = CamWorldPos + WorldDir * SlabTMax;
		}

		if (P.z < (SlabBottomZ+0.01))
		{
			const float2 UV = 0.5 + 0.5 * ((P - RefractionP) / (VisProjectedRayRadius)).xy;
			const int2 Coord = (UV * 64.0f + 0.5f);
			if (all(Coord >= 0) && all(Coord < 64))
			{
				if (Coord.y < 32)
				{
					const float SampleCount = float(SampleCountTexture[Coord].x) * CountScale;
					OutColor = float4(GetHSVDebugColor(SampleCount), 0.0);
				}
				else
				{
					float Weight = GaussianFunction((P.xy - RefractionP.xy) / VisProjectedRayRadius, LobeState.Mean.xy, LobeState.Variance.x);
					OutColor = float4(GetHSVDebugColor(Weight), 0.0);
				}
			}
		}
	}

#if 1
	// Draw projected distribution as a curve
	float3 PrevP = -1.0f;
	float PrevValue = -1.0f;
	for (float u = 0.0; u < 1.0; u += 0.005f)
	{
		float3 P = lerp(SlabP0, SlabP1, u);
		P.y = 0.5 * (SlabP0.y + SlabP1.y);
		P.z = SlabBottomZ;

		const float2 UV = 0.5 + 0.5 * ((P - RefractionP) / (VisProjectedRayRadius)).xy;
		const int2 Coord = (UV * 64.0f + 0.5f);
		if (all(Coord >= 0) && all(Coord < 64))
		{
			const float SampleCount = float(SampleCountTexture[Coord].x) * CountScale;
			const float Value = float(SampleCount);

			if (PrevValue != -1.0f)
			{
				AddLineWS(Context, 
					PrevP +  float3(0, 0, PrevValue), 
					P +  float3(0, 0, Value),
					float4(1, 1, 0, 0.5));
			}

			PrevP = P;
			PrevValue = Value;
		}
	}

	// Draw gaussian curve mapping to recovered variance
	PrevP = -1.0f;
	PrevValue = -1.0f;
	for (float u = 0.0; u < 1.0; u += 0.005f)
	{
		float3 P = lerp(SlabP0, SlabP1, u);
		P.y = 0.5 * (SlabP0.y + SlabP1.y);
		P.z = SlabBottomZ;

		float Weight = GaussianFunction((P.x - RefractionP.x) / VisProjectedRayRadius, LobeState.Mean.x, LobeState.Variance.x);
		const float Value = Weight;

		if (PrevValue != -1.0f)
		{
			AddLineWS(Context,
				PrevP + float3(0, 0, PrevValue),
				P + float3(0, 0, Value),
				float4(0, 1, 0, 0.5));
		}

		PrevP = P;
		PrevValue = Value;
	}
#endif

#if 0
	const float SphereRadius = 10.0f;
	float2 Sol = RayIntersectSphere(CamWorldPos, WorldDir, float4(RefractionP, SphereRadius));
	if (Sol.x > 0.0f && Sol.y > 0.0f)
	{
		const float3 P = CamWorldPos + WorldDir * Sol.x;
		const float3 OmegaOut = P - RefractionP;
		const float OmegaOutLen = length(OmegaOut);
		const float3 OmegaOutNorm = OmegaOut / OmegaOutLen;

		const float NoH = dot(normalize(OmegaOutNorm + UI.V), WOLRD_NORMAL);
		float D = D_GGX(UI.a2, NoH);
		OutColor = float4(D, D, 0.0, 0.0);
	}
#endif

	// Draw the top and bottom interfaces
	float3 SlabOrigin = SlabCenter - 0.5 * (SlabEdgeX + SlabEdgeY + SlabEdgeZ);
	for (uint i = 0; i <= SlabInterfaceLineCount; i++)
	{
		const float Offset = (float(i) / float(SlabInterfaceLineCount));

		AddLineWS(Context, SlabOrigin + SlabEdgeX * Offset, SlabOrigin + SlabEdgeX * Offset + SlabEdgeY, float4(0.2, 0.2, 0.2, 1.0));
		AddLineWS(Context, SlabOrigin + SlabEdgeY * Offset, SlabOrigin + SlabEdgeY * Offset + SlabEdgeX, float4(0.2, 0.2, 0.2, 1.0));

		AddLineWS(Context, SlabOrigin + SlabEdgeX * Offset + SlabEdgeZ, SlabOrigin + SlabEdgeX * Offset + SlabEdgeY + SlabEdgeZ, float4(0.2, 0.2, 0.2, 1.0));
		AddLineWS(Context, SlabOrigin + SlabEdgeY * Offset + SlabEdgeZ, SlabOrigin + SlabEdgeY * Offset + SlabEdgeX + SlabEdgeZ, float4(0.2, 0.2, 0.2, 1.0));
	}

	// Draw the rays
	for (float u = 0.5f; u < TraceDomainSize; u++)
	{
		for (float v = 0.5f; v < TraceDomainSize; v++)
		{
			float2 RandomUV = float2(u, v) / TraceDomainSize;
			float3 L = 0;
			float Value = 0;
			if (GetGGXDirection(RandomUV, UI.Roughness, UI.V, L, Value, UI.Eta12))
			{
				const float3 RayOnPlane = L.xyz / abs(L.z);
				if(Value > 0.0f) AddLineWS(Context, RefractionP, RefractionP + RayOnPlane * VisProjectedRayRadius, float4(0, Value, 0, 0.3));
				//if(Value > 0.0f) AddLineWS(Context, RefractionP + RayOnPlane * VisProjectedRayRadius, float3((RefractionP + L * VisProjectedRayRadius).xy, SlabCenter.z - 0.5* UI.SlabThickness), float4(0, 1, 0, 0.3));
			}
			else
			{
				const float3 RayOnPlane = L.xyz / abs(L.z);
				if (Value > 0.0f) AddLineWS(Context, RefractionP, RefractionP + RayOnPlane * VisProjectedRayRadius, float4(Value, 0, 0, 0.3));
			}
		}
	}

#if 0
	// Draw the mean / variance estimated by Belcour.
	// However, those data cannot be interpreted as-is because they have been linearized for a add operation of variance to lead to a angular distribution of two convolution (w.r.t. 2 roughnesses).
#if MODE_REFLECTION
	FSubstrateLobeStatistic PaperLobe = SubstrateGetReflectedLobe(SubstrateGetDiracLobe(UI.V), 1.0, UI.Roughness);
#else
	FSubstrateLobeStatistic PaperLobe = SubstrateGetRefractedLobe2(SubstrateGetDiracLobe(UI.V), 1.0, UI.Roughness, UI.Eta12);
#endif
	DrawCircleXY(Context, float3(SlabCenter.xy + PaperLobe.Mu.xy * VisProjectedRayRadius, SlabBottomZ), PaperLobe.Sigma * VisProjectedRayRadius, float4(1, 0, 1, 1));

	Newline(Context);
	Newline(Context);
	Print(Context, TEXT("Belcour variance = "));
	Print(Context, PaperLobe.Sigma, FontWhite, 5, 5);
#endif

	// Our mean / variance
	DrawCircleXY(Context, float3(SlabCenter.xy + LobeState.Mean * VisProjectedRayRadius, SlabBottomZ + 0.05), LobeState.Variance * VisProjectedRayRadius, float4(1, 1, 0, 1));

	Newline(Context);
	Newline(Context);
	Print(Context, TEXT("New variance X = "));
	Print(Context, LobeState.Variance.x, FontWhite, 5, 5);
	Newline(Context);
	Print(Context, TEXT("New variance Y = "));
	Print(Context, LobeState.Variance.y, FontWhite, 5, 5);

}

#endif // VISUALIZE_ROUGH_REFRACTION_PS



#endif // SUBSTRATE_RND_SHADERS


