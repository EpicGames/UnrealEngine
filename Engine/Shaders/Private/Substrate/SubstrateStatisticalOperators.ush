// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"

// Statistical operators representing BSDF as a sum of lobes.
// This implementation is based on [Belcour 2018, "Efficient Rendering of Layered Materials using an Atomic Decomposition with Statistical Operators"]

struct FSubstrateLobeStatistic
{
	// Mean
	// xy is the 2d projection of the lobe main direction onto the plane defined by the surface normal. 
	// z is making the 3d vector a unit vector. This is useful to evaluate `s` for refraction operations (equation 10).
	float3 Mu;

	// Energy
	float3 E;

	// Variance
	float Sigma;
};



float SubstrateLobeRoughnessToVariance(float Roughness)
{
	// Eq. 6
	const float SafeRoughness = clamp(Roughness, 0.0f, 0.999f);
	const float a11 = pow(SafeRoughness, 1.1f);
	return a11 / (1.0f - a11);
}

float SubstrateLobeVarianceToRoughness(float Variance)
{
	// Inverse of Eq. 6
	return pow(max(0.f, Variance / (1.0f + Variance)), 1.0f / 1.1f);
}



// For the following statistical operators
//  - WiLobe: the lobe towards which light is reflected
//  - InterfaceRoughness = the material layer roughness
//  - InterfaceFDG = the material layer directional albedo
//  - InterfaceEta12 = the ratio of refractive mediaEta1 / Eta2
//  - OpticalDepth = the value impact light transmission (= sigma_t * depth)
//  - Approximation to in-scattering from the back of the layer (eq.19)

FSubstrateLobeStatistic SubstrateGetNullLobe()
{
	FSubstrateLobeStatistic NullLobe = (FSubstrateLobeStatistic)0;
	return NullLobe;
}

// Wi typically points outward the surface
FSubstrateLobeStatistic SubstrateGetDiracLobe(float3 Wi)
{
	FSubstrateLobeStatistic WiLobe;
	WiLobe.E = 1.0f;
	WiLobe.Mu = Wi;
	WiLobe.Sigma = 0.0f;
	return WiLobe;
}

// This is used when a lob needs to be converted from OmegaO to OmegaI.
// For instance: a lobe OmegaR is reflected from OmegaI on a surface. Then it needs to be refracted from a top interface.
//				We thus need to convert OmegaR to be BottomUpOmegaI relatively to the top surface.
//				This is achieved using BottomUpOmegaI = SubstrateOppositeLobe(OmegaR)
FSubstrateLobeStatistic SubstrateOppositeLobe(FSubstrateLobeStatistic LobeIn)
{
	FSubstrateLobeStatistic LobeOut = LobeIn;
	LobeOut.Mu = -LobeOut.Mu;
	return LobeOut;
}

FSubstrateLobeStatistic SubstrateGetReflectedLobe(FSubstrateLobeStatistic WiLobe, float3 InterfaceFDG, float InterfaceRoughness)
{
	FSubstrateLobeStatistic WoLobe;

	WoLobe.E = WiLobe.E * InterfaceFDG;

	// The lobe remains on the same side of the surface, mirrored over the normal.
	WoLobe.Mu = float3(-WiLobe.Mu.xy, WiLobe.Mu.z);

	WoLobe.Sigma = WiLobe.Sigma + SubstrateLobeRoughnessToVariance(InterfaceRoughness);

	return WoLobe;
}

FSubstrateLobeStatistic SubstrateGetRefractedLobe(FSubstrateLobeStatistic WiLobe, float3 InterfaceFDG, float InterfaceRoughness, float InterfaceEta12)
{
	FSubstrateLobeStatistic WoLobe;

	WoLobe.E = WiLobe.E * (1.0f - InterfaceFDG);

	// The lobe is on the oposite side of the surface, and the direction account for change of medium IOR.
	WoLobe.Mu.xy = -WiLobe.Mu.xy * InterfaceEta12;
	WoLobe.Mu.z  = -sign(WiLobe.Mu.z) * sqrt(1.0 - dot(WoLobe.Mu.xy, WoLobe.Mu.xy)); // derive z from vector projected xy

	//	const float S = 0.5f * (1.0f + InterfaceEta12 * (WiLobe.Mu.z / WoLobe.Mu.z));				// This respect eq.10 but it goes crazy and does not respect the roughness of a single front layer
	//	const float S = 0.5f * (1.0f + InterfaceEta12 * max(0.0, WiLobe.Mu.z / WoLobe.Mu.z));		// This respect the roughness range better but looks incorrect
	const float S = 1.0f;																			// ==> Until this is fully understood, do not scale anything.
	WoLobe.Sigma = (WiLobe.Sigma / InterfaceEta12) + SubstrateLobeRoughnessToVariance(S * InterfaceRoughness);

	return WoLobe;
}

FSubstrateLobeStatistic SubstrateGetTransmittedLobe(FSubstrateLobeStatistic WiLobe, float3 OpticalDepth)
{
	FSubstrateLobeStatistic WoLobe;

	WoLobe.E = WiLobe.E * exp(-OpticalDepth);

	// The lobe is on the oposite side of the surface along -Wi
	WoLobe.Mu = -WiLobe.Mu.xyz;

	WoLobe.Sigma = WiLobe.Sigma;

	return WoLobe;
}

FSubstrateLobeStatistic SubstrateWeightLobe(FSubstrateLobeStatistic A, float Weight)
{
	// We simple weight the lob properties so that they progressively disapear as a function of coverage.
	// This is not entirely correct though because if matter coverage is reduced there should be two lobe: 
	//	- this lobe with visibility = coverage
	//	- a dirac lobe with visibility = 1 - coverage
	FSubstrateLobeStatistic WoLobe = A;
	WoLobe.E *= Weight;
	WoLobe.Mu.xy = WoLobe.Mu.xy * Weight;
	WoLobe.Mu.z = sign(WoLobe.Mu.z) * sqrt(1.0 - dot(WoLobe.Mu.xy, WoLobe.Mu.xy)); // derive z from vector projected xy
	WoLobe.Sigma *= Weight;
	return WoLobe;
}

FSubstrateLobeStatistic SubstrateHorizontalMixLobes(FSubstrateLobeStatistic A, FSubstrateLobeStatistic B, float Mix)
{
	FSubstrateLobeStatistic WoLobe;
	WoLobe.E = lerp(A.E, B.E, Mix);
	WoLobe.Mu = lerp(A.Mu, B.Mu, Mix);
	WoLobe.Mu.z = sign(WoLobe.Mu.z) * sqrt(1.0 - dot(WoLobe.Mu.xy, WoLobe.Mu.xy)); // derive z from vector projected xy
	WoLobe.Sigma = lerp(A.Sigma, B.Sigma, Mix);
	return WoLobe;
}

