// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../TransmissionCommon.ush"
#include "/Engine/Shared/SubstrateDefinitions.h"
#include "/Engine/Private/Substrate/SubstrateExportCommon.ush"

// Forward declarations
uint PackColorLinearToGamma2AlphaLinear(float4 In);
float4 UnpackColorGamma2ToLinearAlphaLinear(uint In);

///////////////////////////////////////////////////////////////////////////////
// Sub-surface 
// 
// Stored in first slice as UINT
// This is hot data that is often accessed by SSS related passes.
struct FSubstrateSubsurfaceHeader
{
	// 3 bits
	// SSS type:
	// 0: Invalid
	// 1: Wrap
	// 2: TwoSided
	// 3: Diffusion
	// 4: Diffusion with profile
	// 5: SimpleVolume
	// 6: Transmittance shadow

	// 29 bits of data.
	// If  TransmittanceShadow	: 10109 mean free path. SUBSTRATE_TODO unify with Substrate per pixel MFP with option for squared transmittance and skip screen space SSS post process. That would then be a MFP of 999bits. NonProfile can also be renamed to SSSPerPixelMFP.
	// If  bIsProfile			: 8bit profile radius, 8bits ProfileId, 16bits unused
	// If !bIsProfile			: 10109 mean free path
	uint Bytes;
};

#if SSS_TYPE_COUNT != 7
#error Update this code to ensure all SSS types requiring side payload can be represented.
#endif
#define SSSHEADER_TYPE(Header)	(Header.Bytes & 0x7)
#define SSSHEADER_TYPE_MASK		(0x00000007)

void SubstrateSubSurfaceHeaderSetSSSType(inout FSubstrateSubsurfaceHeader SSSHeader, uint SSSType)
{
	SSSHeader.Bytes &= (~SSSHEADER_TYPE_MASK);
	SSSHeader.Bytes |= SSSType & SSSHEADER_TYPE_MASK;
}

void SubstrateSubSurfaceHeaderSetProfile(inout FSubstrateSubsurfaceHeader SSSHeader, float RadiusScale, uint ProfileId)
{
	SSSHeader.Bytes &= SSSHEADER_TYPE_MASK;
	SSSHeader.Bytes |= ProfileId << 24;
	SSSHeader.Bytes |= PackR8(RadiusScale) << 16;
}

void SubstrateSubSurfaceHeaderSetNonProfile(inout FSubstrateSubsurfaceHeader SSSHeader, float3 MeanFreePath)
{
	SSSHeader.Bytes &= SSSHEADER_TYPE_MASK;
	// Encode MFP onto 10 10 9 bits. For the blue channel the last bit of the mantissa is dropped in order to make room for SSSType encoding
	SSSHeader.Bytes |= (Pack10F(MeanFreePath.x) << 22) | (Pack10F(MeanFreePath.y) << 12) | ((Pack10F(MeanFreePath.z)&0x3FE) << 2);
}

void SubstrateSubSurfaceHeaderSetTransmittanceShadowMFPInMeters(inout FSubstrateSubsurfaceHeader SSSHeader, float3 MeanFreePath)
{
	SubstrateSubSurfaceHeaderSetNonProfile(SSSHeader, MeanFreePath);
}

float SubstrateSubSurfaceGetWrapOpacityFromAnisotropy(float PhaseAnisotropy)
{
	// Reinterpret the phase function anisotropy as 'opacity' value
	// * Forward/Backward phase function == Thin surface (i.e., opacity => 0)
	// * Isotropic phase function == Thick surface (i.e., opacity => 1)
	const float Opacity = 1.f - abs(PhaseAnisotropy);
	return Opacity;
}

void SubstrateSubSurfaceHeaderSetWrapOpacity(inout FSubstrateSubsurfaceHeader SSSHeader, float Opacity)
{
	SSSHeader.Bytes &= SSSHEADER_TYPE_MASK;
	SSSHeader.Bytes |= PackR8(Opacity) << 8;
}

void SubstrateSubSurfaceHeaderSetWrap(inout FSubstrateSubsurfaceHeader SSSHeader, float PhaseAnisotropy)
{
	const float Opacity = SubstrateSubSurfaceGetWrapOpacityFromAnisotropy(PhaseAnisotropy);
	SubstrateSubSurfaceHeaderSetWrapOpacity(SSSHeader, Opacity);
}

bool SubstrateSubSurfaceHeaderGetIsValid(in FSubstrateSubsurfaceHeader SSSHeader)
{
	return SSSHEADER_TYPE(SSSHeader) != SSS_TYPE_NONE;
}

bool SubstrateSubSurfaceHeaderHasExtras(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const uint SSSType = SSSHEADER_TYPE(SSSHeader);
	return SSSType == SSS_TYPE_DIFFUSION || SSSType == SSS_TYPE_DIFFUSION_PROFILE;
}

bool SubstrateSubSurfaceHeaderGetUseDiffusion(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const uint SSSType = SSSHEADER_TYPE(SSSHeader);
	return SSSType == SSS_TYPE_DIFFUSION || SSSType == SSS_TYPE_DIFFUSION_PROFILE;
}

bool SubstrateSubSurfaceHeaderGetIsProfile(in FSubstrateSubsurfaceHeader SSSHeader)
{
	return SSSHEADER_TYPE(SSSHeader) == SSS_TYPE_DIFFUSION_PROFILE;
}

bool SubstrateSubSurfaceHeaderGetIsWrap(in FSubstrateSubsurfaceHeader SSSHeader)
{
	return SSSHEADER_TYPE(SSSHeader) == SSS_TYPE_WRAP || SSSHEADER_TYPE(SSSHeader) == SSS_TYPE_TWO_SIDED_WRAP;
}

uint SubstrateSubSurfaceHeaderGetSSSType(in FSubstrateSubsurfaceHeader SSSHeader)
{
	return SSSHEADER_TYPE(SSSHeader);
}

uint SubstrateSubSurfaceHeaderGetProfileId(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const uint SSSType = SSSHEADER_TYPE(SSSHeader);
	const uint ProfileId = (SSSType == SSS_TYPE_DIFFUSION_PROFILE) ? ((SSSHeader.Bytes >> 24) & 0xFF) : (SSSType == SSS_TYPE_DIFFUSION ? SSS_PROFILE_ID_PERPIXEL : SSS_PROFILE_ID_INVALID);
	return ProfileId;
}

float SubstrateSubSurfaceHeaderGetProfileRadiusScale(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const bool bIsValidAndProfile = SSSHEADER_TYPE(SSSHeader) == SSS_TYPE_DIFFUSION_PROFILE;
	return bIsValidAndProfile ? UnpackR8(SSSHeader.Bytes >> 16) : 1.f;
}

float3 SubstrateSubSurfaceHeaderGetMFP(in FSubstrateSubsurfaceHeader SSSHeader)				// Only when !Profile (== Substrate SSSPerPixelMFP) or TransmittanceShadow.
{
	// xxxxxxxxxx
	//           xxxxxxxxxx
	//                     xxxxxxxxx
	//                              xxx
	// 10bits    10bits    9bits    3bits
	// MFP.R     MFP.G     MFP.B    SSSType
	return float3(
		Unpack10F(SSSHeader.Bytes >> 22),			// 10bits
		Unpack10F(SSSHeader.Bytes >> 12),			// 10bits
		Unpack10F((SSSHeader.Bytes >> 2) & 0x3FE)); //  9bits
}

float3 SubstrateSubSurfaceHeaderGetTransmittanceShadowMFPInMeters(in FSubstrateSubsurfaceHeader SSSHeader)
{
	return SubstrateSubSurfaceHeaderGetMFP(SSSHeader);
}

float SubstrateSubSurfaceHeaderGetWrapOpacity(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const uint SSSType = SSSHEADER_TYPE(SSSHeader);
	return (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP) ? UnpackR8(SSSHeader.Bytes >> 8) : 1.0f;
}

float SubstrateSubSurfaceHeaderGetOpacity(in FSubstrateSubsurfaceHeader SSSHeader)
{
	const uint SSSType = SSSHEADER_TYPE(SSSHeader);

	// Shadow 'Opacity' is based on legacy shading code
	// * SSS Wrap              -> SubstrateSubSurfaceHeaderGetWrapOpacity()
	// * SSS Diffusion         -> 1
	// * SSS Diffusion Profile -> SubstrateSubSurfaceHeaderGetProfileRadiusScale()
	float Opacity = 1.0f;
	Opacity = SSSType == SSS_TYPE_DIFFUSION_PROFILE ? SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader) : Opacity;
	Opacity = (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP) ? SubstrateSubSurfaceHeaderGetWrapOpacity(SSSHeader) : Opacity;
	return Opacity;
}

// Stored in second slice as UINT
// This is cold data, rarely accessed
struct FSubstrateSubsurfaceExtras
{
	//	float3 BaseColor; // alpha unused
	uint Bytes;
};
void SubstrateSubsurfaceExtrasSetBaseColor(inout FSubstrateSubsurfaceExtras SSSExtras, float3 BaseColor)
{
	SSSExtras.Bytes = PackColorLinearToGamma2AlphaLinear(float4(BaseColor, 0.0f));
}
float3 SubstrateSubsurfaceExtrasGetBaseColor(in FSubstrateSubsurfaceExtras SSSExtras)
{
	return UnpackColorGamma2ToLinearAlphaLinear(SSSExtras.Bytes).rgb;
}

struct FSubstrateSubsurfaceData
{
	FSubstrateSubsurfaceHeader Header;
	FSubstrateSubsurfaceExtras Extras;
};
FSubstrateSubsurfaceHeader SubstrateLoadSubsurfaceHeader(Texture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos)
{
	FSubstrateSubsurfaceHeader Header;
	Header.Bytes = SubstrateBuffer.Load(uint4(PixelPos, FirstSliceStoringSubstrateSSSData + 0, 0));
	return Header;
}
FSubstrateSubsurfaceExtras SubstrateLoadSubsurfaceExtras(Texture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos)
{
	FSubstrateSubsurfaceExtras Extras;
	Extras.Bytes = SubstrateBuffer.Load(uint4(PixelPos, FirstSliceStoringSubstrateSSSData + 1, 0));
	return Extras;
}
FSubstrateSubsurfaceData SubstrateLoadSubsurfaceData(Texture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos)
{
	FSubstrateSubsurfaceData SSSData;
	SSSData.Header = SubstrateLoadSubsurfaceHeader(SubstrateBuffer, FirstSliceStoringSubstrateSSSData, PixelPos);
	SSSData.Extras = SubstrateLoadSubsurfaceExtras(SubstrateBuffer, FirstSliceStoringSubstrateSSSData, PixelPos);
	return SSSData;
}

void SubstrateStoreSubsurfaceHeader(RWTexture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos, uint HeaderBytes, uint QuadPixelWriteMask, uint2 QuadPixelPos)
{
	WriteDataToBuffer(SubstrateBuffer, HeaderBytes, uint3(PixelPos, FirstSliceStoringSubstrateSSSData + 0), QuadPixelWriteMask, QuadPixelPos);
}

void SubstrateStoreSubsurfaceHeader(RWTexture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos, uint HeaderBytes)
{
	SubstrateStoreSubsurfaceHeader(SubstrateBuffer, FirstSliceStoringSubstrateSSSData, PixelPos, HeaderBytes, 1 /*QuadPixelWriteMask*/, PixelPos);
}

void SubstrateStoreSubsurfaceExtras(RWTexture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos, uint ExtraBytes, uint QuadPixelWriteMask, uint2 QuadPixelPos)
{
	WriteDataToBuffer(SubstrateBuffer, ExtraBytes, uint3(PixelPos, FirstSliceStoringSubstrateSSSData + 1), QuadPixelWriteMask, QuadPixelPos);
}

void SubstrateStoreSubsurfaceExtras(RWTexture2DArray<uint> SubstrateBuffer, uint FirstSliceStoringSubstrateSSSData, uint2 PixelPos, uint ExtraBytes)
{
	SubstrateStoreSubsurfaceExtras(SubstrateBuffer, FirstSliceStoringSubstrateSSSData, PixelPos, ExtraBytes, 1/*QuadPixelWriteMask*/, PixelPos);
}

///////////////////////////////////////////////////////////////////////////////
// Shadow and transmission 

// A structure that can be used to transfert Legacy/substrate data for light transmission.
struct FSubsurfaceOpacityMFP
{
	bool  bDataIsOpacity;	// If false, Data is a mean free path
	float Data;				// MFP or Opacity
	float Density;			// Must be stored separately from Opacity, because Opacity is used for some tests
};

FSubsurfaceOpacityMFP GetInitialisedSubsurfaceOpacityMFP()
{
	FSubsurfaceOpacityMFP SubsurfaceOpacityMFP;
	SubsurfaceOpacityMFP.bDataIsOpacity = true;
	SubsurfaceOpacityMFP.Data = 1.0;
	SubsurfaceOpacityMFP.Density = 0.0f;
	return SubsurfaceOpacityMFP;
}

float SubstrateShadowMFPToExtinction(float MFP)
{
	const float Extinction = 1.0f / max(0.00001f, MFP);
	return Extinction;
}

float SubstrateExtinctionToShadowMFP(float Extinction)
{
	const float MFP = 1.0f / max(0.00001f, Extinction);
	return MFP;
}

float SubstrateShadowColoredMFPToGreyScaleMFP(float3 MFP)
{
	return dot(MFP, (1.0f / 3.0f).xxx);
}

// Legacy conversion function, which translated Substrate sub-surface data into 'opacity' for shadow transmission purpose. 
FSubsurfaceOpacityMFP SubstrateGetSubsurfaceOpacityMFP(FSubstrateSubsurfaceHeader SSSHeader, bool bAllowDiffuse = true, bool bAllowDiffuseProfile = true)
{
	FSubsurfaceOpacityMFP SubsurfaceOpacityMFP = GetInitialisedSubsurfaceOpacityMFP();

	const uint SSSType = SubstrateSubSurfaceHeaderGetSSSType(SSSHeader);

	if (bAllowDiffuse && SSSType == SSS_TYPE_DIFFUSION)
	{
		SubsurfaceOpacityMFP.bDataIsOpacity = false;
		const float3 ColoredMFP = SubstrateSubSurfaceHeaderGetMFP(SSSHeader);
		SubsurfaceOpacityMFP.Data = SubstrateShadowColoredMFPToGreyScaleMFP(ColoredMFP);
		SubsurfaceOpacityMFP.Density = 0.0f;
	}
	else if (bAllowDiffuseProfile && SSSType == SSS_TYPE_DIFFUSION_PROFILE)
	{
		SubsurfaceOpacityMFP.bDataIsOpacity = true;

		const float ProfileRadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
		// This clamp aligns with SubsurfaceDensityFromOpacity
		// Various engine paths treat these subsurface materials differently
		// even when they have Opacity = 1 in the material shader, so this is
		// important to avoid things like backface transmission being shadowed by
		// contact shadows and so on.
		const float Opacity = min(ProfileRadiusScale, 0.99f);
		SubsurfaceOpacityMFP.Data = Opacity;

		FTransmissionProfileParams TransmissionParams = GetTransmissionProfileParams(SubstrateSubSurfaceHeaderGetProfileId(SSSHeader));
		SubsurfaceOpacityMFP.Density = SubsurfaceDensityFromOpacity(SubsurfaceOpacityMFP.Data) * TransmissionParams.ExtinctionScale;
	}
	else if (SSSType == SSS_TYPE_WRAP || SSSType == SSS_TYPE_TWO_SIDED_WRAP)
	{
		SubsurfaceOpacityMFP.bDataIsOpacity = true;

		// This clamp aligns with SubsurfaceDensityFromOpacity
		// Various engine paths treat these subsurface materials differently
		// even when they have Opacity = 1 in the material shader, so this is
		// important to avoid things like backface transmission being shadowed by
		// contact shadows and so on.
		const float Opacity = min(SubstrateSubSurfaceHeaderGetWrapOpacity(SSSHeader), 0.99f);
		SubsurfaceOpacityMFP.Data = Opacity;
		SubsurfaceOpacityMFP.Density = SubsurfaceDensityFromOpacity(SubsurfaceOpacityMFP.Data);
	}

	return SubsurfaceOpacityMFP;
}

FTransmissionProfileParams GetSubstrateSubsurfaceTransmissionProfileParams(uint InSSSType, FSubstrateSubsurfaceHeader InSSSHeader)
{
	FTransmissionProfileParams Out = InitTransmissionProfileParams();
	if (InSSSType == SSS_TYPE_DIFFUSION_PROFILE)
	{
		Out = GetTransmissionProfileParams(SubstrateSubSurfaceHeaderGetProfileId(InSSSHeader));
	}
	else if (InSSSType == SSS_TYPE_DIFFUSION)
	{
		//For per pixel diffusion, use default transmission parameters (only NormalScale & ExtinctionScale are used)
		const float3 ColoredMFP = SubstrateSubSurfaceHeaderGetMFP(InSSSHeader);
		const float GreyMFP    = SubstrateShadowColoredMFPToGreyScaleMFP(ColoredMFP);
		const float Extinction  = SubstrateShadowMFPToExtinction(GreyMFP);
		Out.ExtinctionScale = Extinction;
	}
	return Out;
}