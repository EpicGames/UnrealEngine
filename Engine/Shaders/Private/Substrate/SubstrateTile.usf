// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/SubstrateDefinitions.h"   
#if SUBSTRATE_CLASSIFICATION_TILE_SIZE_16
#undef SUBSTRATE_TILE_SIZE
#undef SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT
#define SUBSTRATE_TILE_SIZE							16
#define SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT			4
#endif
#include "SubstrateTile.ush"


////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_TILE_VS

Buffer<uint> TileListBuffer;
uint		 TileListBufferOffset;
uint		 TileEncoding;

void SubstrateTilePassVS(
	in uint InVertexId : SV_VertexID,
	in uint InInstanceId : SV_InstanceID,
#if PERMUTATION_ENABLE_TEXCOORD_SCREENVECTOR
	out float2 OutTexCoord : TEXCOORD0,		// Used by DeferredLightPixelMain directional light
	out float3 OutScreenVector : TEXCOORD1,	// Used by DeferredLightPixelMain directional light
#endif
#if PERMUTATION_ENABLE_DEBUG
	out uint2 OutTileCoord : TILE_COORD,	// Used by StencilMainPS below
#endif
	out float4 OutPosition : SV_POSITION)	// Use by StencilMainPS
{
#if SUBSTRATE_ENABLED
	const uint2 TileCoord = SubstrateUnpackTile(TileListBuffer[TileListBufferOffset + InInstanceId], TileEncoding);
#else
	// Not used, but this shader is compiled for non-Substrate path for setup reason, but not used
	const uint2 TileCoord = 0;
#endif

	uint2 TileVertex = TileCoord * SUBSTRATE_TILE_SIZE;
	TileVertex.x += InVertexId == 1 || InVertexId == 2 || InVertexId == 4 ? SUBSTRATE_TILE_SIZE : 0;
	TileVertex.y += InVertexId == 2 || InVertexId == 4 || InVertexId == 5 ? SUBSTRATE_TILE_SIZE : 0;

	OutPosition = float4(float2(TileVertex) * View.ViewSizeAndInvSize.zw * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);

#if PERMUTATION_ENABLE_TEXCOORD_SCREENVECTOR
	OutTexCoord = (TileVertex + View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
	OutScreenVector = ScreenVectorFromScreenRect(float4(OutPosition.xy, 1, 0));
#endif

#if PERMUTATION_ENABLE_DEBUG
	OutTileCoord = TileCoord;
#endif
}

#endif // SHADER_TILE_VS

////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if SHADER_STENCIL_TAGGING_PS

float4 DebugTileColor;

void StencilTaggingMainPS(
	in uint2 InTileCoord : TILE_COORD,
	in float4 SVPos : SV_POSITION,
	out float4 OutColor0 : SV_Target0)
{
	const bool bTileX = (InTileCoord.x & 1) == 0;
	const bool bTileY = (InTileCoord.y & 1) == 0;
	const bool bChecker = (bTileX && bTileY) || (!bTileX && !bTileY);
	OutColor0 = DebugTileColor * float4(1.0f, 1.0f, 1.0f, bChecker ? 0.33f : 0.66f);
	OutColor0.rgb *= OutColor0.a; // premultiplied alpha blending assuming alpha is opacity==coverage
}

#endif // SHADER_STENCIL_PS


