// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Shared/SubstrateDefinitions.h"
#include "Substrate.ush"

#if SUBSTRATE_ENABLED

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Substrate material type tile

uint2 SubstrateUnpackTile(uint In, uint InEncoding)
{
	return InEncoding == SUBSTRATE_TILE_ENCODING_16BITS ? UnpackTileCoord16bits(In) : UnpackTileCoord8bits(In);
}

uint SubstratePackTile(uint2 TileCoord, uint InEncoding)
{
	return InEncoding == SUBSTRATE_TILE_ENCODING_16BITS ? PackUInt2ToUInt(TileCoord)/*TODO*/ : PackTileCoord8bits(TileCoord);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Substrate Closure tile

struct FSubstrateClosureTile
{
	uint2 TileCoord;
	uint  ClosureIndex;
	uint  ClosureCount;
};

uint PackClosureTile(FSubstrateClosureTile In)
{
	// Tile coord are encoded onto 10bits. Max resolution 8k x 8k. Max 8 tiles per pixel.
	return 
		PackTileCoord10bits(In.TileCoord)|
		((In.ClosureIndex & 0xF  ) << 20)| 
		((In.ClosureCount & 0xF  ) << 24);
}

FSubstrateClosureTile UnpackClosureTile(uint In)
{
	FSubstrateClosureTile Out;
	Out.TileCoord   = UnpackTileCoord10bits(In);
	Out.ClosureIndex= (In >> 20u) & 0xF;
	Out.ClosureCount= (In >> 24u) & 0xF;
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Substrate Closure offset

// Store the material slice index offset for each closure, for pixels containing several closures
struct FSubstrateClosureOffset
{
	uint ClosureOffsets[SUBSTRATE_MATERIAL_CLOSURE_COUNT];
	uint ClosureCount;
};

bool HasMultipleClosures(uint InPackedClosureOffset)
{
	// If a tile has:
	// * No closure its value will be 0 (no offset)
	// * 1 closure it will have a single offset (value=1, 1uint header)
	// * Several closures, it will have several offset (value > 1)
	return InPackedClosureOffset > 1;
}

uint PackClosureOffset(FSubstrateClosureOffset In)
{
	// Packed as relative cumulative offsets of N bits, stored into 32 bits
	// | N bits | N bits | N bits | N bits | ... | N bits |
	//   Clos 0   Clos 1   Clos 2   Clos 3  ...   Clos (2^N - 1)

	uint Out = 0;
	const uint ClosureCount = min(In.ClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);
	#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
	if (ClosureCount > 0)
	{
		Out = In.ClosureOffsets[0] & SUBSTRATE_CLOSURE_OFFSET_BIT_MASK;
		UNROLL_N(SUBSTRATE_MATERIAL_CLOSURE_COUNT-1)
		for (uint i = 1; i < ClosureCount; i++)
		{
			const uint Curr = In.ClosureOffsets[i];
			const uint Prev = In.ClosureOffsets[i-1];
			Out = Out | (((Curr - Prev) & SUBSTRATE_CLOSURE_OFFSET_BIT_MASK) << (SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT * i));
		}
	}
	#endif
	return Out;
}

FSubstrateClosureOffset UnpackClosureOffset(uint In, uint InClosureCount)
{
	FSubstrateClosureOffset Out = (FSubstrateClosureOffset)0;
	if (InClosureCount > 0)
	{
		Out.ClosureCount = min(InClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);
		Out.ClosureOffsets[0] = (In & SUBSTRATE_CLOSURE_OFFSET_BIT_MASK);

		#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
		UNROLL_N(SUBSTRATE_MATERIAL_CLOSURE_COUNT-1)
		for (uint i = 1; i < Out.ClosureCount; i++)
		{
			Out.ClosureOffsets[i] = Out.ClosureOffsets[i-1] + ((In>>(SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT * i)) & SUBSTRATE_CLOSURE_OFFSET_BIT_MASK);
		}
		#endif
	}
	return Out;
}

uint UnpackClosureOffsetAtIndex(uint In, uint InIndex, uint InClosureCount)
{
	uint Out = 0;
	#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
	if (InClosureCount > 0)
	{
		const uint OffsetCount = min3(InIndex+1, InClosureCount, SUBSTRATE_MATERIAL_CLOSURE_COUNT);
		UNROLL_N(SUBSTRATE_MATERIAL_CLOSURE_COUNT)
		for (uint i = 0; i < OffsetCount; i++)
		{
			Out += ((In >> (SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT * i)) & SUBSTRATE_CLOSURE_OFFSET_BIT_MASK);
		}
	}
	#endif
	return Out;
}

void SubstrateSeekClosure(inout FSubstrateAddressing Addressing, uint AddressOffset)
{
	Addressing.CurrentIndex = AddressOffset;
	Addressing.ReadBytes = Addressing.CurrentIndex * 4;
}

void SubstrateSeekClosure(inout FSubstrateAddressing Addressing, FSubstrateClosureOffset Offsets, uint InClosureIndex)
{
	if (InClosureIndex < Offsets.ClosureCount)
	{
		Addressing.CurrentIndex = Offsets.ClosureOffsets[InClosureIndex];
		Addressing.ReadBytes = Addressing.CurrentIndex * 4;
	}
}

#endif // SUBSTRATE_ENABLED
