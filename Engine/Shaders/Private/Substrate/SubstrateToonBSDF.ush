// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if SUBSTRATE_ENABLED

#include "../Common.ush"
#include "../BRDF.ush"
#include "../AreaLightCommon.ush"

#include "../ToonProfileCommon.ush"

#include "/Engine/Shared/SubstrateDefinitions.h"


uint PackToonCustomData(in float2 UVs, in float ProfileId, in float Dither)
{
	uint U12bits = saturate(UVs.x) * 4095.0f;
	uint V12bits = saturate(UVs.y) * 4095.0f;

	uint Packed = U12bits;
	Packed |= V12bits << 12;
	Packed |= (uint(ProfileId) & 0xFF) << 24;

	return Packed;
}


void UnpackToonCustomData(in uint PackedToonCustomData, inout float2 UVs, inout float ProfileId)
{
	uint U12bits = float(BitFieldExtractU32(PackedToonCustomData, 12, 0));
	uint V12bits = float(BitFieldExtractU32(PackedToonCustomData, 12, 12));

	UVs = float2(U12bits, V12bits) * (1.0f / 4095.0f);

	ProfileId = float(BitFieldExtractU32(PackedToonCustomData, 8, 24)); // Stays in [0, 255]
}


// Data to forward to FSubstrateEvaluateResult or FDirectLighting
struct FSubstrateEvaluateToonResult
{
	float3 IntegratedDiffuseValue;
	float3 IntegratedSpecularValue;

	float3 DiffuseColor;

	float3 EmissivePathValue;

	float3 DiffusePathValue;
	float3 SpecularPathValue;

	float  DiffusePDF;

	float3 ThroughputV;
	float3 TransmittanceAlongN;
	bool   bPostProcessSubsurface;

	float ToonDataWeight;
	float ToonDataShadowNoL;
};

// PatternsUVs can only be used on mobile when the extended GBuffer is enabled. Always available on non mobile renderer.
// When PatternsUVs cannot be used, ramp uvs and shadow hatching cannot be renderered and are skipped.
#define TOON_CAN_USE_PATTERNS_UVS (!SHADING_PATH_MOBILE || (!MOBILE_DEFERRED_SHADING || MOBILE_EXTENDED_GBUFFER))

FSubstrateEvaluateToonResult SubstrateToonEvaluateCommon(
	float3	DiffuseColor,
	float3	SpecularColor,
	float	Roughness,
	float	ToonProfileId,
	float2	ToonPatternUVs,
	float	Anisotropy,
	float3	Emissive,
	BxDFContext Context,
	bool bHasAnisotropy,
	bool bIsRectLight,
	bool bThisPassAppliesDiffuseRamp,
	half3 N, half3 V, FAreaLight AreaLight, 
	float ShadowTermsSurfaceShadow,		// ShadowTerms.SurfaceShadow
	float ShadowTermsTransmittance)		// ShadowTerms.TransmittanceOrOpticalThickness, storing transmittance in the case of toon.
{
	FSubstrateEvaluateToonResult Sample = (FSubstrateEvaluateToonResult)0;

	FToonProfileEvaluateSettings ToonEvalSettings = GetToonProfileEvaluateSettings(ToonProfileId);

	float ToonProfileDiffuseDither = 0.0f;
	float ToonProfileSpecularDither = 0.0f;
	float TransmittanceShadow = ShadowTermsTransmittance;
#if TOON_CAN_USE_PATTERNS_UVS
	ToonProfileDiffuseDither = EvaluateToonProfileDiffuseDither(ToonProfileId, ToonPatternUVs, ToonEvalSettings);
	ToonProfileSpecularDither = EvaluateToonProfileSpecularDither(ToonProfileId, ToonPatternUVs, ToonEvalSettings);

	// Apply shadow/transmittance hatching pattern
	FToonProfileShadowSettings ToonProfileShadowSettings = GetToonProfileShadowSettings(ToonProfileId);
	TransmittanceShadow = EvaluateToonProfileShadowHatching(ToonProfileId, ToonPatternUVs, ToonProfileShadowSettings, TransmittanceShadow);
#endif

	float3 Specular = 0.0f;
	{
		const float SafeRoughness = MakeRoughnessSafe(Roughness, SUBSTRATE_MIN_GGX_ROUGHNESS);
		float Alpha2 = Pow4(SafeRoughness);
		float Alpha2Spec = Alpha2;

		const float3 F0 = SpecularColor;
		const float3 F90 = 1.0f;

		if (bIsRectLight)
		{
			// In this case, we set D to 1 and Vis will contain the area light / GGX integration
			float3 RectLightSpec = RectGGXApproxLTC(SafeRoughness, /*F0*/ SpecularColor, /*F90*/ 1.0f, N, V, AreaLight.Rect, AreaLight.Texture);

			Specular = RectLightSpec;
		}
		else
		{
			float D;
			if (bHasAnisotropy && Anisotropy != 0)
			{
				float2 AlphaXY = 0;
				float  Alpha = Square(SafeRoughness);
				GetAnisotropicRoughness(Alpha, Anisotropy, AlphaXY.x, AlphaXY.y);
				D = D_GGXaniso(AlphaXY.x, AlphaXY.y, Context.NoH, Context.XoH, Context.YoH);
			}
			else
			{
				D = D_GGX(Alpha2Spec, Context.NoH);
			}
			//const float3 FresnelTerm = Substrate_F_GGX(F0, F90, Context.VoH);
			//float Vis = Substrate_Vis_GGX(SafeRoughness, Alpha2Spec, Context.NoV, AreaLight.NoL);

			Specular = D;// * Vis * FresnelTerm;
			Specular *= GGXMaxEnergyLookup(SafeRoughness, Anisotropy, Context.NoV);
		}

		// TODO H_PDF and PDF
	}

	// Diffuse
	float RampDiffuseNoL = Context.NoL * 0.5 + 0.5;
	if (ToonEvalSettings.bDiffuseRampIncludeShadow)
	{
		RampDiffuseNoL = RampDiffuseNoL * TransmittanceShadow;
	}
	const float3 DiffuseRamp = bThisPassAppliesDiffuseRamp ? EvaluateToonProfileDiffuseRamp(ToonProfileId, RampDiffuseNoL + ToonProfileDiffuseDither) : 1.0f;
	Sample.DiffuseColor = DiffuseColor * DiffuseRamp;
	Sample.DiffusePathValue = Diffuse_Lambert(Sample.DiffuseColor);
	Sample.IntegratedDiffuseValue = (ToonEvalSettings.bDiffuseRampIncludeShadow ? 1.0f : TransmittanceShadow) * AreaLight.Falloff * AreaLight.FalloffColor * Sample.DiffusePathValue; //  * AreaLight.NoL for smoother edges? But then transmittance is blocked. So this is left in the hand of the artist.

	// Toon unified diffse data.
	Sample.ToonDataWeight = TransmittanceShadow * AreaLight.Falloff;	// Accumulate light weight. Albedo should not drive light importance so we do not need to include Lambert diffuse here.
	Sample.ToonDataShadowNoL = RampDiffuseNoL;							// NoL with or without shadow, remapped to [0,1] range.

	// Specular
	const float3 SpecularRamp = SpecularColor * EvaluateToonProfileSpecularRamp(ToonProfileId, saturate(dot(Specular, 1.0f / 3.0f) + ToonProfileSpecularDither));
	Sample.SpecularPathValue = SpecularRamp;
	Sample.IntegratedSpecularValue += ShadowTermsSurfaceShadow * AreaLight.NoL * AreaLight.Falloff * AreaLight.FalloffColor * Sample.SpecularPathValue;

	// Others
	Sample.DiffusePDF = saturate(Context.NoL) / PI;
	Sample.bPostProcessSubsurface = false;

	Sample.EmissivePathValue = Emissive;

	const float OpaqueBSDFThroughput = 0.0f;
	Sample.ThroughputV = OpaqueBSDFThroughput;
	Sample.TransmittanceAlongN = OpaqueBSDFThroughput;

	return Sample;

}

#endif // SUBSTRATE_ENABLED

