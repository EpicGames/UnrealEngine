// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#include "/Engine/Private/ShadingCommon.ush"

#ifndef SUBSTRATE_OPAQUE_MATERIAL
#error SUBSTRATE_OPAQUE_MATERIAL not defined
#endif

// SUBSTRATE_TODO We assume layers all have the same IOR. But we should take into account the material later.
// For now, we suppose no IOR changes at interface until they are tracked properly
#define WATER_IOR		1.0f 
#define AIR_IOR			1.0f

///////////////////////////////////////////////////////////////////////////////
// Tree processing functions
#if ALLOW_SUBSTRATE_TREE

#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
void FSubstrateTree::UpdateSingleBSDFOperatorCoverageTransmittance(
	FSubstratePixelHeader SubstratePixelHeader,
	int BSDFIndex,
	FSubstrateIntegrationSettings Settings,
	float3 V)
#else
void UpdateSingleBSDFOperatorCoverageTransmittance(
	inout FSubstrateTree InSubstrateTree,
	FSubstratePixelHeader SubstratePixelHeader,
	int BSDFIndex,
	FSubstrateIntegrationSettings Settings,
	float3 V)
#endif
{
#if SUBSTRATE_INLINE_SHADING

#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
	#define CurrentBSDF this.BSDFs[BSDFIndex]
	#define Op this.Operators[CurrentBSDF.OperatorIndex]
#else
	#define CurrentBSDF InSubstrateTree.BSDFs[BSDFIndex]
	#define Op InSubstrateTree.Operators[CurrentBSDF.OperatorIndex]
#endif

	// this feature is only needed for development/editor - we can compile it out for a shipping build (see r.CompileShadersForDevelopment cvar help)
#if USE_DEVELOPMENT_SHADERS
	// We progressively remove top layers but never the bottom one.
	CurrentBSDF.Coverage = int(Op.LayerDepth) < Settings.PeelLayersAboveDepth && !CurrentBSDF.bIsBottom ? 0.0f : CurrentBSDF.Coverage;
#endif

	float CurrentBSDFCoverage = CurrentBSDF.Coverage;
	float3x3 TangentBasis = SubstrateGetBSDFSharedBasis_InlineShading(SubstratePixelHeader, CurrentBSDF);

	float Roughness = SubstrateGetBSDFRoughness(CurrentBSDF);
#if SUBSTRATE_FASTPATH==0
	// Combine second roughness to allso affect opqaue and translucent rough refraction
	const bool bHasHaziness = BSDF_GETHASHAZINESS(CurrentBSDF);
	BRANCH
	if (bHasHaziness)
	{
		FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(CurrentBSDF));
		const bool  bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;
		if (!bHazeAsSimpleClearCoat)
		{
			const float HazeWeight = Haziness.Weight;
			const float HazeSafeRoughness = MakeRoughnessSafe(Haziness.Roughness);
			Roughness = lerp(Roughness, HazeSafeRoughness, HazeWeight);
		}
	}
#endif

#if USE_DEVELOPMENT_SHADERS
	Roughness = Settings.bRoughnessTracking ? Roughness : 0.0;
#endif

	// Sanitize BSDF before it is used for forward shading or exported to GBuffer
	CurrentBSDF.SubstrateSanitizeBSDF();

	// Distortion and translucency passes accumnulate pass calls for colored transmittance, coverage and TopDownRefractionLobeStat to be setup correctly.
#if (SUBSTRATE_FASTPATH==0 || MATERIALBLENDING_ANY_TRANSLUCENT) && !SUBSTRATE_OPTIMIZED_UNLIT // !Fast path or rendering translucent materials

	// Extract the true thickness of the BSDF before it is cast into the normalised Slab thickness we used.
	// for opaque rough refraction, we are only interested in thickness of non bottom layers.
	Op.OpaqueRoughRefractThicknessCm  = CurrentBSDF.bIsBottom ? 0.0f					: CurrentBSDF.ThicknessCm;
	Op.OpaqueRoughRefractCoverage	  = CurrentBSDF.bIsTop    ? CurrentBSDF.Coverage	: 0.0f;
	Op.OpaqueRoughRefractTopRoughness = CurrentBSDF.bIsTop    ? Roughness				: 0.0f;

	const bool bIsSubstrateOpaqueMaterial = SUBSTRATE_OPAQUE_MATERIAL > 0;
	CurrentBSDF.PostProcessBSDFBeforeLighting(bIsSubstrateOpaqueMaterial);

	// Compute current BSDF transmittance
	float3 LEqualN = TangentBasis[2];
	FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;
	FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(SubstratePixelHeader, CurrentBSDF, NullSubstrateAddressing, V, LEqualN);
	FSubstrateEvaluateResult BSDFEvaluate = SubstrateEvaluateBSDF(SubstrateBSDFContext, Settings);

	// We multiply this weight with coverage:
	//  - Because when coming from parameter blending, we need to scale the output down according to coverage resulting from parameter blending.
	//  - That will be applied on emissive and reflected light, so we do not need to apply that anywhere else.
	//  - When coming from non parameter blending, this is equal identity 1 and the operator parsing will continue to correctly weight the BSDF according to the tree.
	CurrentBSDF.LuminanceWeightV = float3(1.0f, 1.0f, 1.0f) * CurrentBSDFCoverage;

	// Initialise to "no matter above"
	CurrentBSDF.CoverageAboveAlongN = 0.0f;
	CurrentBSDF.TransmittanceAboveAlongN = float3(1.0f, 1.0f, 1.0f);

	// All BSDF are considered for contribution as the top BSDF can have their coverage dip to 0 and then another BSDF need to be considered for the top layer representation.
	CurrentBSDF.TopLayerDataWeight = 1.0f;

	Op.Coverage				= CurrentBSDFCoverage;
	Op.ThroughputAlongV		= BSDFEvaluate.ThroughputV;
	Op.TransmittanceAlongN	= BSDFEvaluate.TransmittanceAlongN;

	// Evaluate refraction lobes.
	const FBxDFEnergyTerms EnergyTerms = ComputeGGXSpecEnergyTerms(Roughness, SubstrateBSDFContext.SatNoV, SubstrateGetBSDFSpecularF0(CurrentBSDF), SubstrateGetBSDFSpecularF90(CurrentBSDF));
	{
		const float TopDownInterfaceEta = CurrentBSDF.bIsTop ? AIR_IOR / WATER_IOR : 1.0f;
		FSubstrateLobeStatistic RefractedLobe = SubstrateGetRefractedLobe(SubstrateGetDiracLobe(V), EnergyTerms.E, Roughness, TopDownInterfaceEta);
		// Account for transmittance of the medium (but ignoring scattering for now)
		RefractedLobe.E *= BSDFEvaluate.ThroughputV;
		// Reduce according to coverage. It is a approximation to handle rough refraction reduction according to coverage when relying on a single lob.
		RefractedLobe = SubstrateWeightLobe(RefractedLobe, CurrentBSDFCoverage);
		Op.TopDownRefractionLobeStat = RefractedLobe;
	}

	// Same operation but for a bottom up lob
	{
		const float BottomUpInterfaceEta = CurrentBSDF.bIsTop ? WATER_IOR / AIR_IOR : 1.0f;
		const float3 NormalDown = float3(0, 0, -1);
		FSubstrateLobeStatistic RefractedLobe = SubstrateGetRefractedLobe(SubstrateGetDiracLobe(NormalDown), EnergyTerms.E, Roughness, BottomUpInterfaceEta);
		// Account for transmittance of the medium (but ignoring scattering for now)
		RefractedLobe.E *= BSDFEvaluate.ThroughputV;
		// Reduce according to coverage. It is a approximation to handle rough refraction reduction according to coverage when relying on a single lob.
		RefractedLobe = SubstrateWeightLobe(RefractedLobe, CurrentBSDFCoverage);
		Op.BottomUpRefractionLobeStat = RefractedLobe;
	}

#elif SUBSTRATE_OPTIMIZED_UNLIT

	CurrentBSDF.LuminanceWeightV		= CurrentBSDFCoverage;
	CurrentBSDF.CoverageAboveAlongN		= 0.0f;
	CurrentBSDF.TransmittanceAboveAlongN= float3(1.0f, 1.0f, 1.0f);
	CurrentBSDF.TopLayerDataWeight		= 1.0f;

	Op.OpaqueRoughRefractThicknessCm	= 0.0;
	Op.OpaqueRoughRefractCoverage		= CurrentBSDFCoverage;
	Op.OpaqueRoughRefractTopRoughness	= Roughness;

	Op.Coverage				= CurrentBSDFCoverage;
	Op.ThroughputAlongV		= UNLIT_TRANSMITTANCE(CurrentBSDF);
	Op.TransmittanceAlongN	= UNLIT_TRANSMITTANCE(CurrentBSDF);

	Op.TopDownRefractionLobeStat = SubstrateGetNullLobe();
	Op.BottomUpRefractionLobeStat = SubstrateGetNullLobe();

#else //

	CurrentBSDF.LuminanceWeightV		= CurrentBSDFCoverage;
	CurrentBSDF.CoverageAboveAlongN		= 0.0f;
	CurrentBSDF.TransmittanceAboveAlongN= float3(1.0f, 1.0f, 1.0f);
	CurrentBSDF.TopLayerDataWeight		= 1.0f;

	Op.OpaqueRoughRefractThicknessCm	= 0.0;
	Op.OpaqueRoughRefractCoverage		= CurrentBSDFCoverage;
	Op.OpaqueRoughRefractTopRoughness	= Roughness;

	Op.Coverage				= CurrentBSDFCoverage;
	Op.ThroughputAlongV		= 1.0f;
	Op.TransmittanceAlongN	= 1.0f;

	Op.TopDownRefractionLobeStat = SubstrateGetNullLobe();
	Op.BottomUpRefractionLobeStat = SubstrateGetNullLobe();

#endif // SUBSTRATE_COMPLEXPATH
	
	Op.TopDownRefractionWorldNormal = TangentBasis[2];

#undef Op
#undef CurrentBSDF

#endif // SUBSTRATE_INLINE_SHADING
}


void FSubstrateTree::UpdateSingleOperatorCoverageTransmittance(int OpIndex)
{
#if SUBSTRATE_INLINE_SHADING

#define Op this.Operators[OpIndex]
#define OpA this.Operators[this.Operators[OpIndex].LeftIndex]
#define OpB this.Operators[this.Operators[OpIndex].RightIndex]

	// Propagate bIsTop:
	//  - horizontal, add, weight operations: Any of the operand bIsTop should be enough to determine this operator depth
	//	- vertical: takes the top layer depth information (operand A)
	// => we always take the operand A information.
	Op.LayerDepth = OpA.LayerDepth;

	switch (Op.Type)
	{

	case SUBSTRATE_OPERATOR_WEIGHT:
	{
		const float Weight = saturate(Op.Weight);
		Op.Coverage				= Weight * OpA.Coverage;
		Op.ThroughputAlongV		= OpA.ThroughputAlongV;
		Op.TransmittanceAlongN	= OpA.TransmittanceAlongN;

		Op.TopDownRefractionLobeStat = SubstrateWeightLobe(OpA.TopDownRefractionLobeStat, Weight);		// This is used for rough refraction and we weight energy and roughness to make sure coverage from weight node can make rough refraction vanish.
		Op.TopDownRefractionWorldNormal = OpA.TopDownRefractionWorldNormal * Weight;

		Op.BottomUpRefractionLobeStat = SubstrateWeightLobe(OpA.BottomUpRefractionLobeStat, Weight);	// This is used for opaue rough refraction, we scale the lobe to make it vanish in order to have a smooth continuous blur behavior.

		// Coverage does not affect rough refraction data
		Op.OpaqueRoughRefractThicknessCm  = OpA.OpaqueRoughRefractThicknessCm;
		Op.OpaqueRoughRefractCoverage     = OpA.OpaqueRoughRefractCoverage * Weight;
		Op.OpaqueRoughRefractTopRoughness = OpA.OpaqueRoughRefractTopRoughness;
		break;
	}

#if !SUBSTRATE_OPTIMIZED_UNLIT

	case SUBSTRATE_OPERATOR_VERTICAL:
	{
		const float3 TopThroughputAlongV	= OpA.ThroughputAlongV;
		const float3 TopTransmittanceAlongN = OpA.TransmittanceAlongN;
		const float3 BotThroughputAlongV	= OpB.ThroughputAlongV;
		const float3 BotTransmittanceAlongN = OpB.TransmittanceAlongN;
		const float TopCoverage = OpA.Coverage;
		const float BotCoverage = OpB.Coverage;

#if 0
		// This is coverage assuming correlation
		const float LayerMaxCoverage = max(TopCoverage, BotCoverage);
		const float LayerMaxCoverageInv = LayerMaxCoverage <= 0.0f ? 1.0f : 1.0f / LayerMaxCoverage;

		// If a layer has a lower coverage than the other, we account for that when compuing the Transmittance.
		const float TransmittanceOne = 1.0f;
		const float3 TopTransmittanceAdjustedV = lerp(TransmittanceOne, TopThroughputAlongV,	TopCoverage * LayerMaxCoverageInv);
		const float3 TopTransmittanceAdjustedL = lerp(TransmittanceOne, TopTransmittanceAlongN, TopCoverage * LayerMaxCoverageInv);
		const float3 BotTransmittanceAdjustedV = lerp(TransmittanceOne, BotThroughputAlongV,	BotCoverage * LayerMaxCoverageInv);
		const float3 BotTransmittanceAdjustedL = lerp(TransmittanceOne, BotTransmittanceAlongN, BotCoverage * LayerMaxCoverageInv);

		// Now we update the material transmittance and coverage
		Op.Coverage = LayerMaxCoverage;
		Op.ThroughputAlongV		= TopTransmittanceAdjustedV * BotTransmittanceAdjustedV;
		Op.TransmittanceAlongN	= TopTransmittanceAdjustedL * BotTransmittanceAdjustedL;
#else
		FVerticalLayeringInfo Info = GetVerticalLayeringInfo(TopCoverage, BotCoverage);

		Op.Coverage				= Info.Coverage;
		Op.ThroughputAlongV		= Info.TransmittanceOnlyTop * TopThroughputAlongV +		Info.TransmittanceOnlyBottom * BotThroughputAlongV +	Info.TransmittanceTopAndBottom * TopThroughputAlongV * BotThroughputAlongV;
		Op.TransmittanceAlongN  = Info.TransmittanceOnlyTop* TopTransmittanceAlongN + Info.TransmittanceOnlyBottom * BotTransmittanceAlongN + Info.TransmittanceTopAndBottom * TopTransmittanceAlongN * BotTransmittanceAlongN;
#endif

		Op.VerticalTop_Coverage				= OpA.Coverage;
		Op.VerticalTop_ThroughputAlongV		= OpA.ThroughputAlongV;
		Op.VerticalTop_TransmittanceAlongN  = OpA.TransmittanceAlongN;

		Op.VerticalBot_Coverage				= OpB.Coverage;
		Op.VerticalBot_ThroughputAlongV		= OpB.ThroughputAlongV;
		Op.VerticalBot_TransmittanceAlongN	= OpB.TransmittanceAlongN;

		// Project top lob through bottom layer assuming medium have same IOR.
		{
			const float TopDownInterfaceEta = 1.0f;
			Op.TopDownRefractionLobeStat = SubstrateGetRefractedLobe(SubstrateOppositeLobe(OpA.TopDownRefractionLobeStat), 
				/*InterfaceFDG*/0.0f, SubstrateLobeVarianceToRoughness(OpB.TopDownRefractionLobeStat.Sigma), TopDownInterfaceEta);
			Op.TopDownRefractionLobeStat.E *= saturate(OpB.ThroughputAlongV + (1.0f - OpB.Coverage)); // Combine with botton layer medium and specular transmittance

			// The ratio of BottomSurfaceOnlyVisibile and visible through top layer (with a scale of 0.5 because at maximum both surfaces will be visible) as a fonction of the total coverage.
			const float RefractionNormalMix = saturate((Info.SurfaceBottom + 0.5f * Info.TransmittanceTopAndBottom * dot(TopThroughputAlongV, 0.33.xxx)) / max(SUBSTRATE_EPSILON, Info.Coverage));
			Op.TopDownRefractionWorldNormal = lerp(OpA.TopDownRefractionWorldNormal, OpB.TopDownRefractionWorldNormal, RefractionNormalMix);
		}

		// Project bottom lob through top layer.
		{
			const bool bIsTop = Op.LayerDepth == 0;
			const float BottomUpInterfaceEta = bIsTop ? WATER_IOR / AIR_IOR : 1.0f;
			Op.BottomUpRefractionLobeStat = SubstrateGetRefractedLobe(SubstrateOppositeLobe(OpB.BottomUpRefractionLobeStat), 
				/*InterfaceFDG*/0.0f, SubstrateLobeVarianceToRoughness(OpA.BottomUpRefractionLobeStat.Sigma), BottomUpInterfaceEta);
			Op.BottomUpRefractionLobeStat.E *= saturate(OpA.ThroughputAlongV + (1.0f - OpA.Coverage)); // Combine with top layer medium and specular transmittance
		}

		// Vertical layering accumulates both operators thickness and only keep the top layer roughness and coverage, see OpaqueRoughRefractTopRoughness declaration for the details.
		Op.OpaqueRoughRefractThicknessCm  = OpA.OpaqueRoughRefractThicknessCm + OpB.OpaqueRoughRefractThicknessCm;
		Op.OpaqueRoughRefractCoverage     = OpA.OpaqueRoughRefractCoverage;
		Op.OpaqueRoughRefractTopRoughness = OpA.OpaqueRoughRefractTopRoughness;

		Op.VerticalTop_TopDownRefractionLobeStat = OpA.TopDownRefractionLobeStat;
		Op.VerticalTop_BottomUpRefractionLobeStat = OpA.BottomUpRefractionLobeStat;
		break;
	}

	case SUBSTRATE_OPERATOR_HORIZONTAL:
	{
		const float Mix = saturate(Op.Weight);
		const float AMix = 1.0 - Mix;
		const float BMix = Mix;

		Op.Coverage = AMix * OpA.Coverage + BMix * OpB.Coverage;
		Op.ThroughputAlongV    = (AMix * OpA.Coverage * OpA.ThroughputAlongV    + BMix * OpB.Coverage * OpB.ThroughputAlongV)    / max(SUBSTRATE_EPSILON, AMix * OpA.Coverage + BMix * OpB.Coverage);
		Op.TransmittanceAlongN = (AMix * OpA.Coverage * OpA.TransmittanceAlongN + BMix * OpB.Coverage * OpB.TransmittanceAlongN) / max(SUBSTRATE_EPSILON, AMix * OpA.Coverage + BMix * OpB.Coverage);

		Op.TopDownRefractionLobeStat = SubstrateHorizontalMixLobes(OpA.TopDownRefractionLobeStat, OpB.TopDownRefractionLobeStat, BMix);
		Op.TopDownRefractionWorldNormal = lerp(OpA.TopDownRefractionWorldNormal, OpB.TopDownRefractionWorldNormal, BMix);

		Op.BottomUpRefractionLobeStat = SubstrateHorizontalMixLobes(OpA.BottomUpRefractionLobeStat, OpB.BottomUpRefractionLobeStat, BMix);

		// Horizontal mixes both operators thickness
		Op.OpaqueRoughRefractThicknessCm  = lerp(OpA.OpaqueRoughRefractThicknessCm,  OpB.OpaqueRoughRefractThicknessCm,  BMix);
		Op.OpaqueRoughRefractCoverage     = lerp(OpA.OpaqueRoughRefractCoverage,     OpB.OpaqueRoughRefractCoverage,     BMix);
		Op.OpaqueRoughRefractTopRoughness = lerp(OpA.OpaqueRoughRefractTopRoughness, OpB.OpaqueRoughRefractTopRoughness, BMix);
		break;
	}

	case SUBSTRATE_OPERATOR_ADD:
	{
		const float SafeABSDFCoverage = saturate(OpA.Coverage);
		const float SafeBBSDFCoverage = saturate(OpB.Coverage);
		const float AMixFactor = SafeABSDFCoverage / max(SUBSTRATE_EPSILON, SafeABSDFCoverage + SafeBBSDFCoverage);

		Op.Coverage = saturate(OpA.Coverage + OpB.Coverage);
		Op.ThroughputAlongV    = lerp(OpB.ThroughputAlongV,    OpA.ThroughputAlongV,    AMixFactor);
		Op.TransmittanceAlongN = lerp(OpB.TransmittanceAlongN, OpA.TransmittanceAlongN, AMixFactor);

		Op.TopDownRefractionLobeStat = SubstrateHorizontalMixLobes(OpA.TopDownRefractionLobeStat, OpB.TopDownRefractionLobeStat, 0.5f);
		Op.TopDownRefractionWorldNormal = lerp(OpA.TopDownRefractionWorldNormal, OpB.TopDownRefractionWorldNormal, 0.5f);

		Op.BottomUpRefractionLobeStat = SubstrateHorizontalMixLobes(OpA.BottomUpRefractionLobeStat, OpB.BottomUpRefractionLobeStat, 0.5f);

		Op.OpaqueRoughRefractThicknessCm  = lerp(OpA.OpaqueRoughRefractThicknessCm,  OpB.OpaqueRoughRefractThicknessCm,  0.5f);
		Op.OpaqueRoughRefractCoverage     = lerp(OpA.OpaqueRoughRefractCoverage,     OpB.OpaqueRoughRefractCoverage,     0.5f);
		Op.OpaqueRoughRefractTopRoughness = lerp(OpA.OpaqueRoughRefractTopRoughness, OpB.OpaqueRoughRefractTopRoughness, 0.5f);
		break;
	}

#endif // !SUBSTRATE_OPTIMIZED_UNLIT

	}

#undef Op
#undef OpA
#undef OpB

#endif // SUBSTRATE_INLINE_SHADING
}



#define CurrentBSDF this.BSDFs[BSDFIndex]
#define Op this.Operators[OpIndex]

// Post-update visit operators
void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Weight(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA)
{
#if SUBSTRATE_INLINE_SHADING

	const float Weight = saturate(Op.Weight);
	CurrentBSDF.LuminanceWeightV *= Weight;
	CurrentBSDF.Coverage *= Weight;

	CurrentBSDF.TopLayerDataWeight *= Weight;

#endif // SUBSTRATE_INLINE_SHADING
}

void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Horizontal(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA)
{
#if SUBSTRATE_INLINE_SHADING

	const float Mix = saturate(Op.Weight);
	const float AMix = 1.0 - Mix;
	const float BMix = Mix;
	const float Weight = PreviousIsInputA > 0 ? AMix : BMix;
	CurrentBSDF.LuminanceWeightV *= Weight;
	CurrentBSDF.Coverage *= Weight;

	CurrentBSDF.TopLayerDataWeight *= Weight;

#endif // SUBSTRATE_INLINE_SHADING
}

void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Vertical(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA)
{
#if SUBSTRATE_INLINE_SHADING

	// PreviousIsInputA > 0 means it is a BSDF affecting the top layer so we do not affect it by anything for this vertical layering.
	const bool bBSDFComesFromTopLayer = PreviousIsInputA > 0;

	// Otherise, the BSDF comes from the bottom part so we affect it weights from the gathered top layer coverage/transmittance
	CurrentBSDF.LuminanceWeightV		 *= bBSDFComesFromTopLayer ? 1.0f : saturate(1.0f - Op.VerticalTop_Coverage) + saturate(Op.VerticalTop_Coverage) * Op.VerticalTop_ThroughputAlongV;

	// Now we are going to combine the coverage/transmittance above this slab into a new one according to the vertical operator we traverse now, if PreviousIsInputA > 0
	const float BotCover  = CurrentBSDF.CoverageAboveAlongN;
	const float TopCover  = Op.VerticalTop_Coverage;
	const float3 BotTrans = CurrentBSDF.TransmittanceAboveAlongN;
	const float3 TopTrans = Op.VerticalTop_TransmittanceAlongN;
	FVerticalLayeringInfo Info = GetVerticalLayeringInfo(Op.VerticalTop_Coverage, CurrentBSDF.CoverageAboveAlongN);
	// Pre coverage transmittance is the surface transmitance as if coverage=1, so the final combined transmittance is renormalized accordingly.
	const float3 PreCoverageTransmittance = saturate((Info.TransmittanceOnlyTop * TopTrans + Info.TransmittanceOnlyBottom * BotTrans + Info.TransmittanceTopAndBottom * (TopTrans * BotTrans)) / max(SUBSTRATE_EPSILON, Info.Coverage));
	CurrentBSDF.CoverageAboveAlongN		  = bBSDFComesFromTopLayer ? CurrentBSDF.CoverageAboveAlongN		: Info.Coverage;			// Info.Coverage combines the incoming operator Top part coverage and the BSDF current CoverageAboveAlongN
	CurrentBSDF.TransmittanceAboveAlongN  = bBSDFComesFromTopLayer ? CurrentBSDF.TransmittanceAboveAlongN	: PreCoverageTransmittance;	// PreCoverageTransmittance is the combined trnasmittance for the BSDF TransmittanceAboveAlongN computed so far and the top layer of this new vertical operator.

	// If the BSDF is from the bottom, we reduce its contribution according to the top layer coverage only.
	CurrentBSDF.TopLayerDataWeight *= bBSDFComesFromTopLayer ? 1.0 : saturate(1.0f - Op.VerticalTop_Coverage);

#if SUBSTRATE_MATERIAL_ROUGHNESS_TRACKING
	if (!bBSDFComesFromTopLayer)	// Only apply to BSDF from the bottom layer
	{
		// We propagate the bottom layer lob up after the top layer top layer simulate the fact that light rays are potentially heavily scattered/refracted by the top layer.
		// The outcome is that we affect the bottom layer roughness to aggregate that fact.
		// SUBSTRATE_TODO: take into account thickness (far-field for now).

		// Evaluate the reflected lobe, as an approximation we only evaluate along the normal.
		const float3 DummyWi = float3(0, 0, 1);
		const float DummyInterfaceDFG = 0.5f;
		FSubstrateLobeStatistic ReflectedLobe = SubstrateGetReflectedLobe(SubstrateGetDiracLobe(DummyWi), DummyInterfaceDFG, SubstrateGetBSDFRoughness(CurrentBSDF));

		// Evaluate the reflected lobe after refraction through the top layer of the vertical operator.
		// => This tells how light should be integrated when traveling up thorugh that layer.
		// However, light should only refract and spread from roughness according to the amount of coverage, and only if transmittance is > 0.
		const float DiracLobeSigma = 0.0f;
		const float TopLayerSigma = lerp(DiracLobeSigma, Op.VerticalTop_BottomUpRefractionLobeStat.Sigma, Op.VerticalTop_Coverage * dot(Op.VerticalTop_TransmittanceAlongN, (1.0f/3.0f).xxx));
		const bool bIsTop = Op.LayerDepth == 0;
		const float BottomUpInterfaceEta = bIsTop ? WATER_IOR / AIR_IOR : 1.0f;
		FSubstrateLobeStatistic RefractedLobe = SubstrateGetRefractedLobe(SubstrateOppositeLobe(ReflectedLobe), DummyInterfaceDFG, SubstrateLobeVarianceToRoughness(TopLayerSigma), BottomUpInterfaceEta);

		CurrentBSDF.SubstrateSetBSDFRoughness(SubstrateLobeVarianceToRoughness(RefractedLobe.Sigma));
	}
#endif // SUBSTRATE_MATERIAL_ROUGHNESS_TRACKING

#endif // SUBSTRATE_INLINE_SHADING
}

#undef Op
#undef CurrentBSDF

#else // ALLOW_SUBSTRATE_TREE

#if SUBSTRATE_COMPILER_SUPPORTS_STRUCT_FORWARD_DECLARATION
void FSubstrateTree::UpdateSingleBSDFOperatorCoverageTransmittance(
	FSubstratePixelHeader SubstratePixelHeader,
	int BSDFIndex,
	FSubstrateIntegrationSettings Settings,
	float3 V){}
#else
void UpdateSingleBSDFOperatorCoverageTransmittance(
	inout FSubstrateTree InSubstrateTree,
	FSubstratePixelHeader SubstratePixelHeader,
	int BSDFIndex,
	FSubstrateIntegrationSettings Settings,
	float3 V){}
#endif

void FSubstrateTree::UpdateSingleOperatorCoverageTransmittance(
	int OpIndex){}

void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Weight(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA){}

void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Horizontal(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA){}

void FSubstrateTree::UpdateAllBSDFWithBottomUpOperatorVisit_Vertical(
	int BSDFIndex,
	int OpIndex,
	int PreviousIsInputA){}
#endif // ALLOW_SUBSTRATE_TREE