// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#if defined(SHADER_DEBUGSUBSTRATETREE_PS) || defined(SHADER_DEBUGSUBSTRATETREE_CS)
#define SUBSTRATE_INLINE_SHADING 1
#else
#define SUBSTRATE_INLINE_SHADING 0
#endif

#define SUBSTRATE_VISUALIZATION_SOURCE 1
#define SUBSTRATE_SSS_MATERIAL_OVERRIDE 0
#define SUBSTRATE_SSS_TRANSMISSION 0				// Disable for now, as the SSS profile texture need to be bound
#define SUBSTRATE_COMPLEXSPECIALPATH 1				// Allow to read data from complex special path
#include "/Engine/Private/DeferredShadingCommon.ush"
#include "/Engine/Private/Substrate/SubstrateEvaluation.ush"
#include "/Engine/Private/Substrate/SubstrateTile.ush"
#include "/Engine/Shared/SubstrateVisualizeDefinitions.h"

#if SUBSTRATE_INLINE_SHADING == 0
#include "/Engine/Private/Substrate/SubstrateVisualizeCommon.ush"
#endif

#include "/Engine/Private/ShaderPrint.ush"
#include "../ColorMap.ush"

uint bOverrideCursorPosition;
int2 GetCursorPosition()
{
	const int2 OverrideCursorPosition = int2((View.ViewSizeAndInvSize.xy + View.ViewRectMin.xy) / 2.0f);
	return bOverrideCursorPosition > 0 ? OverrideCursorPosition : View.CursorPosition;	
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Material Print

#if SHADER_MATERIALPRINT

float3 GetWorldPositionFromPixelPos(uint2 PixelPos)
{
	const float2 BufferUV = SvPositionToBufferUV(float4(PixelPos + 0.5f, 0, 0));
	const float2 ScreenPosition = SvPositionToScreenPosition(float4(PixelPos, 0.5f, 1)).xy;
	const float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
	const float SceneDepth = ConvertFromDeviceZ(DeviceZ);
	const float3 WorldPosition = mul(float4(GetScreenPositionForProjectionType(ScreenPosition, SceneDepth), SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	return WorldPosition;
}

[numthreads(1, 1, 1)]
void MaterialPrintInfoCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 PixelPos = uint2(float2(GetCursorPosition()) * View.ViewResolutionFraction);
	const float3 WorldPosition = GetWorldPositionFromPixelPos(PixelPos);
	const float3 V = -GetCameraVectorFromTranslatedWorldPosition(WorldPosition);

	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

	// Print pixel material properties. Simple print for simple compilers such as dx11/fxc which cannot handle loop...
#if SUBSTRATE_MAX_CLOSURE_COUNT!= 8
#error Unsupported SUBSTRATE_MAX_CLOSURE_COUNT in visualisation shader.
#endif
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 0);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 1);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 2);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 3);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 4);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 5);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 6);
	SubstratePrintMaterialProperties(Context, PixelPos, WorldPosition, V, 7);

	// Serialise out to buffer for print.
	SerializeSubstratePixelDebugDataEntry(PixelPos, PrimaryView.FrameNumber);

	// Print pixel footprint
	const float3 WorldPositionX = GetWorldPositionFromPixelPos(PixelPos + uint2(1, 0));
	const float3 WorldPositionY = GetWorldPositionFromPixelPos(PixelPos + uint2(0, 1));
	const float3 dPdx = WorldPositionX - WorldPosition;
	const float3 dPdy = WorldPositionY - WorldPosition;
	DrawPixelFootprint(WorldPosition, dPdx, dPdy, PixelPos);

	// When overriding cursor position (on platform without mouse), display a small rect to show the pointing position
	if (bOverrideCursorPosition)
	{
		const uint2 CursorPosition = GetCursorPosition();
		FShaderPrintContext Ctx = InitShaderPrintContext(true, uint2(0, 0));
		AddQuadSS(Ctx, float2(CursorPosition) - float2(1, 1), float2(CursorPosition) + float2(1, 1), ColorYellow);
	}
}
#endif // SHADER_MATERIALPRINT

///////////////////////////////////////////////////////////////////////////////////////////////////
// Material Visualize

#if SHADER_MATERIALCOUNT

uint ViewMode;
uint bRealTimeUpdate;

float3 BytePerPixelToColor(uint In)
{
	float3 Out = 0;
	// 16->32  : Blue -> green
	// 32->64  : Green -> Orange
	// 64->128 : Orange -> Red
	if (In < 32)
	{
		Out = ColorMapTurbo(saturate((In-16.f) / 32.f) * 0.25f + 0.25f);
	}
	else if (In < 64)
	{
		Out = ColorMapTurbo(saturate((In-32.f) / 32.f) * 0.25f + 0.5f);
	}
	else
	{
		Out = ColorMapTurbo(saturate((In-64.f) / 64.f) * 0.25f + 0.75f);
	}

	return Out;
}

void VisualizeMaterialPS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	const uint2 PixelPosDynRes = uint2(float2(SVPos.xy) * (bRealTimeUpdate > 0 ? View.ViewResolutionFraction : 1.0f));
	float2 BufferUV = SvPositionToBufferUV(float4(PixelPosDynRes, SVPos.zw));

	const float2 ScreenPosition = SvPositionToScreenPosition(SVPos).xy;
	const float2 ScreenMaterialPosition = SvPositionToScreenPosition(float4(PixelPosDynRes, 0.5f, 1)).xy;
	float DeviceZ = SampleDeviceZFromSceneTextures(BufferUV);
	float SceneDepth = ConvertFromDeviceZ(DeviceZ);
	const float3 WorldPosition = mul(float4(ScreenMaterialPosition * SceneDepth, SceneDepth, 1), PrimaryView.ScreenToTranslatedWorld).xyz;
	const float3 V = -GetCameraVectorFromTranslatedWorldPosition(WorldPosition);


#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPosDynRes, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
#endif

	OutColor = 0;
	OutColor.w = 0.1f;
	if (ViewMode == 2)
	{
		// BSDF count
		const float MaxClosureCount = SUBSTRATE_MAX_CLOSURE_COUNT;
#if SUBSTRATE_GBUFFER_FORMAT==1
		if (Header.ClosureCount > 0)
		{
			const float BSDFCost = saturate(float(Header.ClosureCount) / MaxClosureCount);
			OutColor.xyz = ColorMapTurbo(BSDFCost);
		}
#else
		const float BSDFCost = saturate(1.0f / MaxClosureCount);
		OutColor.xyz = ColorMapTurbo(BSDFCost);
#endif
	
		// Print Legend
		if (all(uint2(SVPos.xy) == uint2(0,0)))
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));
			Print(Context, TEXT("1 BSDF"), InitFontColor(ColorMapTurbo(1 / MaxClosureCount))); Newline(Context);
			Print(Context, TEXT("2 BSDF"), InitFontColor(ColorMapTurbo(2 / MaxClosureCount))); Newline(Context);
			Print(Context, TEXT("3 BSDF"), InitFontColor(ColorMapTurbo(3 / MaxClosureCount))); Newline(Context);
			Print(Context, TEXT("4 BSDF"), InitFontColor(ColorMapTurbo(4 / MaxClosureCount))); Newline(Context);
		}
	}
	else if (ViewMode == 3)
	{
		uint MaterialBytePerPixel = 0;
#if SUBSTRATE_GBUFFER_FORMAT==1
		Substrate_for (uint ClosureIndex = 0, ClosureIndex < Header.ClosureCount, ++ClosureIndex)
		{
			// Unpack BSDF data
			FSubstrateBSDF BSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, Header);
			MaterialBytePerPixel = SubstrateAddressing.ReadBytes;
		}
		
		if (MaterialBytePerPixel > 0)
		{
			OutColor.xyz = BytePerPixelToColor(MaterialBytePerPixel);
		}
#else
		MaterialBytePerPixel = 4 * 4; // 4 32bits gbuffer
		OutColor.xyz = BytePerPixelToColor(MaterialBytePerPixel);
#endif
	
		// Print Legend
		if (all(uint2(SVPos.xy) == uint2(0,0)))
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));
			Print(Context, TEXT("16  Bytes"), InitFontColor(BytePerPixelToColor(16))); Newline(Context);	
			Print(Context, TEXT("32  Bytes"), InitFontColor(BytePerPixelToColor(32))); Newline(Context);
			Print(Context, TEXT("64  Bytes"), InitFontColor(BytePerPixelToColor(64))); Newline(Context);
			Print(Context, TEXT("96  Bytes"), InitFontColor(BytePerPixelToColor(96))); Newline(Context);
			Print(Context, TEXT("128 Bytes"), InitFontColor(BytePerPixelToColor(128))); Newline(Context);
		}

		// Cursor
		if (all(PixelPosDynRes == GetCursorPosition()))
		{
			FShaderPrintContext Context = InitShaderPrintContext(true, uint2(250, 50));
			Print(Context, MaterialBytePerPixel, InitFontColor(BytePerPixelToColor(MaterialBytePerPixel)), 3, 0); 
			Print(Context, TEXT("/"), FontWhite); 
			Print(Context, Substrate.MaxBytesPerPixel, FontYellow, 3, 0); 
			Print(Context, TEXT(" Bytes"), FontWhite); 
		}
	}
}
#endif // SHADER_MATERIALCOUNT

///////////////////////////////////////////////////////////////////////////////////////////////////
// Advanced Material Visualize

#define BSDF_ELEMENT_SIZE 210
#define BSDF_ELEMENT_STARTX 100.0f + 450.0f
#define BSDF_ELEMENT_STARTY 100

#if SHADER_DEBUGSUBSTRATETREE_PS || SHADER_DEBUGSUBSTRATETREE_CS

#define TREE_MAX_DEPTH			4
#define TREE_START_POS			float2(1300, 350)
#define TREE_BRANCH				float2(100, 75)
#define TREE_COLOR				float4(0.8, 0.8, 0.8, 1.0)
#define TREE_NODE_RADIUS		20.0

float2 GetBranchOffset(FSubstrateOperator Op, bool bIsLeft)
{
	if (Op.Type == SUBSTRATE_OPERATOR_WEIGHT)
	{
		return float2(0, TREE_BRANCH.y);
	}
	return TREE_BRANCH * (bIsLeft ? float2(-1, 1) : float2(1, 1));
}

#endif // SHADER_DEBUGSUBSTRATETREE_PS || SHADER_DEBUGSUBSTRATETREE_CS


#if SHADER_DEBUGSUBSTRATETREE_CS

[numthreads(1, 1, 1)]
void MaterialDebugSubstrateTreeCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
// UE-194712 - Temporary workaround with Vulkan shader compilation
#if !COMPILER_VULKAN && !COMPILER_FXC
	const uint2 PixelPos = (float2(GetCursorPosition()) * View.ViewResolutionFraction);

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	if (Header.ClosureCount > 0)
	{
		const int SliceStoringDebugSubstrateTreeData = Substrate.SliceStoringDebugSubstrateTreeData;

		FSubstrateTreeHeader SubstrateTreeHeader = (FSubstrateTreeHeader)0;
		SubstrateUnpackInSubstrateTreeHeader(
			Substrate.MaterialTextureArray,
			SubstrateTreeHeader,
			SliceStoringDebugSubstrateTreeData);

		FShaderPrintContext Context;
		Context = InitShaderPrintContext(true, uint2(50, 50));

		const int BSDFCount = SubstrateTreeHeader.BSDFCount;

		Print(Context, BSDFCount, FontYellow, 2, 0);
		Print(Context, TEXT("BSDFs"), FontYellow);
		if (SubstrateTreeHeader.ConvertedFromLegacy)
		{
			Print(Context, TEXT(" (Converted from legacy material)"), FontOrange);
		}
		Newline(Context);
		Print(Context, SubstrateTreeHeader.OperatorCount, FontWhite, 2, 0);
		Print(Context, TEXT("Operators"), FontWhite);

		#define MAX_LAYER_COUNT 4
		float LayerAccCoverage[MAX_LAYER_COUNT];
		{
			for (int i = 0; i < MAX_LAYER_COUNT; ++i)
			{
				LayerAccCoverage[i] = 0.0f;
			}
		}

		Context = InitShaderPrintContext(true, uint2(50, BSDF_ELEMENT_STARTY));
		{
			SubstrateTreeHeader.OperatorCount = min(SubstrateTreeHeader.OperatorCount, 2);
			SUBSTRATE_UNROLL_N(2)
			for (int i = 0; i < SubstrateTreeHeader.OperatorCount; i++)
			{
				FSubstrateOperator Op = SubstrateUnpackInSubstrateTreeOperator(
					i,
					Substrate.MaterialTextureArray,
					SubstrateTreeHeader,
					SliceStoringDebugSubstrateTreeData);

				Print(Context, TEXT("OperatorIndex = "), FontCyan);
				Print(Context, i, FontCyan, 2, 0);
				Newline(Context);
				Print(Context, TEXT("ParentIndex = "), FontWhite);
				Print(Context, Op.ParentIndex, FontWhite, 2, 0);
				Newline(Context);
				Print(Context, TEXT("LayerDepth = "), FontWhite);
				Print(Context, Op.LayerDepth, FontWhite, 2, 0);
				Newline(Context);
				Print(Context, TEXT("MaxDistanceFromLeaves = "), FontWhite);
				Print(Context, Op.MaxDistanceFromLeaves, FontWhite, 2, 0);
				Newline(Context);
				Print(Context, TEXT("Type = "), FontWhite);
				if (Op.Type == SUBSTRATE_OPERATOR_WEIGHT)
				{
					Print(Context, TEXT("WEIGHT"), FontWhite);
				}
				else if (Op.Type == SUBSTRATE_OPERATOR_VERTICAL)
				{
					Print(Context, TEXT("VERTICAL"), FontWhite);
				}
				else if (Op.Type == SUBSTRATE_OPERATOR_HORIZONTAL)
				{
					Print(Context, TEXT("HORIZONTAL"), FontWhite);
				}
				else if (Op.Type == SUBSTRATE_OPERATOR_ADD)
				{
					Print(Context, TEXT("ADD"), FontWhite);
				}
				else if (Op.Type == SUBSTRATE_OPERATOR_BSDF)
				{
					Print(Context, TEXT("BSDF"), FontWhite);
				}
				else if (Op.Type == SUBSTRATE_OPERATOR_BSDF_LEGACY)
				{
					Print(Context, TEXT("BSDF_LEGACY"), FontWhite);
				}
				else
				{
					Print(Context, TEXT("UNKOWN"), FontWhite);
				}
				Newline(Context);
				if (Op.Type == SUBSTRATE_OPERATOR_BSDF)
				{
					Print(Context, TEXT("BSDFIndex = "), FontYellow);
					Print(Context, Op.LeftIndex, FontYellow, 2, 0);
				}
				else
				{
					Print(Context, TEXT("LeftIndex = "), FontWhite);
					Print(Context, Op.LeftIndex, FontWhite, 2, 0);
				}
				Newline(Context);
				Print(Context, TEXT("RightIndex = "), FontWhite);
				Print(Context, Op.RightIndex, FontWhite, 2, 0);
				Newline(Context);
				Print(Context, TEXT("Weight = "), FontWhite);
				Print(Context, Op.Weight, FontWhite, 5, 2);
				Newline(Context);
				Print(Context, TEXT("Coverage = "), FontWhite);
				Print(Context, Op.Coverage, FontWhite, 5, 2);
				Newline(Context);
				Newline(Context);
			}
		}

		{
			SubstrateTreeHeader.BSDFCount = min(SubstrateTreeHeader.BSDFCount, 2);
			SUBSTRATE_UNROLL_N(2)
			for (int i = 0; i < SubstrateTreeHeader.BSDFCount; i++)
			{
				FSubstrateBSDF BSDF = SubstrateUnpackInSubstrateTreeBSDF(
					i,
					Substrate.MaterialTextureArray,
					SubstrateTreeHeader,
					SliceStoringDebugSubstrateTreeData);

				const bool bActive = any(BSDF.LuminanceWeightV > 0.0);
				FFontColor BSDFFont = FontDarkRed;

				Context = InitShaderPrintContext(true, uint2(300, BSDF_ELEMENT_STARTY + BSDF_ELEMENT_SIZE * i));
				Print(Context, TEXT("BSDFIndex = "), FontYellow);
				Print(Context, i, FontYellow, 2, 0);
				if (!bActive)
				{
					Print(Context, TEXT("DISABLED"), FontDarkRed);
				}
				else
				{
					BSDFFont = FontWhite;
				}
				Newline(Context);
				Print(Context, TEXT("OperatorIndex = "), FontCyan);
				Print(Context, BSDF.OperatorIndex, FontCyan, 2, 0);
				Newline(Context);
				Print(Context, TEXT("Coverage = "), BSDFFont);
				Print(Context, BSDF.Coverage, BSDFFont, 5, 2);
				Newline(Context);
				Print(Context, TEXT("LumWghtV = "), BSDFFont);
				Print(Context, BSDF.LuminanceWeightV.r, FontLightRed, 5, 2);
				Print(Context, BSDF.LuminanceWeightV.g, FontLightGreen, 5, 2);
				Print(Context, BSDF.LuminanceWeightV.b, FontLightBlue, 5, 2);
				Newline(Context);
				Print(Context, TEXT("TopTrans = "), BSDFFont);
				Print(Context, BSDF.TransmittanceAboveAlongN.r, FontLightRed, 5, 2);
				Print(Context, BSDF.TransmittanceAboveAlongN.g, FontLightGreen, 5, 2);
				Print(Context, BSDF.TransmittanceAboveAlongN.b, FontLightBlue, 5, 2);
				Newline(Context);
				Print(Context, TEXT("TopCover = "), BSDFFont);
				Print(Context, BSDF.CoverageAboveAlongN, FontLightRed, 5, 2);
				Newline(Context);
				Newline(Context);
				Print(Context, TEXT("Diffuse  = "), BSDFFont);
				const float3 BSDFDiffuse = SubstrateGetBSDFDiffuseColor(BSDF);
				Print(Context, BSDFDiffuse.r, FontLightRed, 5, 2);
				Print(Context, BSDFDiffuse.g, FontLightGreen, 5, 2);
				Print(Context, BSDFDiffuse.b, FontLightBlue, 5, 2);
				Newline(Context);
			}
		}

		Context = InitShaderPrintContext(true, uint2(675, BSDF_ELEMENT_STARTY - 50));
		Print(Context, TEXT("BSDF visualization"), FontWhite);

		Context = InitShaderPrintContext(true, uint2(1165, BSDF_ELEMENT_STARTY - 50));
		Print(Context, TEXT("Topology"), FontWhite);

		// Debug print the Substrate tree only for DXC for the sake of compilation performance.
	#if SUBSTRATE_COMPILER
		uint NodeIndexStack[TREE_MAX_DEPTH];
		uint ChildrenVisitedStack[TREE_MAX_DEPTH]; // 0:none, 1:left, 2:left&right
		float2 PrintPosStack[TREE_MAX_DEPTH];

		int StackPtr = 0;
		NodeIndexStack[StackPtr] = SubstrateTreeHeader.RootOperatorIndex;
		ChildrenVisitedStack[StackPtr] = 0;
		PrintPosStack[StackPtr] = TREE_START_POS;

		while (StackPtr >= 0)
		{
			uint OperatorIndex = NodeIndexStack[StackPtr];
			FSubstrateOperator Op = SubstrateUnpackInSubstrateTreeOperator(
				OperatorIndex,
				Substrate.MaterialTextureArray,
				SubstrateTreeHeader,
				SliceStoringDebugSubstrateTreeData);

			Context = InitShaderPrintContext(true, uint2(PrintPosStack[StackPtr]));
			if (Op.Type == SUBSTRATE_OPERATOR_WEIGHT)
			{
				Print(Context, TEXT("W"), FontWhite);
			}
			else if (Op.Type == SUBSTRATE_OPERATOR_VERTICAL)
			{
				Print(Context, TEXT("V"), FontWhite);
			}
			else if (Op.Type == SUBSTRATE_OPERATOR_HORIZONTAL)
			{
				Print(Context, TEXT("H"), FontWhite);
			}
			else if (Op.Type == SUBSTRATE_OPERATOR_ADD)
			{
				Print(Context, TEXT("A"), FontWhite);
			}
			else if (Op.Type == SUBSTRATE_OPERATOR_BSDF)
			{
				Context = InitShaderPrintContext(true, uint2(PrintPosStack[StackPtr]) - uint2(3,0));
				Print(Context, TEXT("B"), FontYellow);
				Print(Context, Op.LeftIndex, FontYellow);
			}
			else if (Op.Type == SUBSTRATE_OPERATOR_BSDF_LEGACY)
			{
				Print(Context, TEXT("BL"), FontWhite);
			}
			else
			{
				Print(Context, TEXT("UNKOWN"), FontWhite);
			}

			const float2 ParentPrintPos = PrintPosStack[StackPtr];

			AddCircleSS(Context, ParentPrintPos, TREE_NODE_RADIUS, TREE_COLOR);

			if (Op.LayerDepth == TREE_MAX_DEPTH - 1 || Op.Type == SUBSTRATE_OPERATOR_BSDF || Op.Type == SUBSTRATE_OPERATOR_BSDF_LEGACY)
			{
				StackPtr--;
				continue;
			}

			if (ChildrenVisitedStack[StackPtr] == 0)
			{
				if (Op.LeftIndex >= 0)
				{
					ChildrenVisitedStack[StackPtr] = 1;

					StackPtr++;
					NodeIndexStack[StackPtr] = Op.LeftIndex;
					ChildrenVisitedStack[StackPtr] = 0;

					const float2 BranchOffset = GetBranchOffset(Op, true);
					const float2 BranchOffsetNorm = normalize(BranchOffset);
					PrintPosStack[StackPtr] = ParentPrintPos + BranchOffset;
					AddLineSS(Context, ParentPrintPos + TREE_NODE_RADIUS * BranchOffsetNorm, PrintPosStack[StackPtr] - TREE_NODE_RADIUS * BranchOffsetNorm, TREE_COLOR);
				}
				else if (Op.RightIndex >= 0)
				{
					ChildrenVisitedStack[StackPtr] = 2;

					StackPtr++;
					NodeIndexStack[StackPtr] = Op.RightIndex;
					ChildrenVisitedStack[StackPtr] = 0;

					const float2 BranchOffset = GetBranchOffset(Op, false);
					const float2 BranchOffsetNorm = normalize(BranchOffset);
					PrintPosStack[StackPtr] = ParentPrintPos + BranchOffset;
					AddLineSS(Context, ParentPrintPos + TREE_NODE_RADIUS * BranchOffsetNorm, PrintPosStack[StackPtr] - TREE_NODE_RADIUS * BranchOffsetNorm, TREE_COLOR);
				}
				else
				{
					StackPtr--;
				}
			}
			else if (ChildrenVisitedStack[StackPtr] == 1)
			{
				if (Op.RightIndex >= 0)
				{
					ChildrenVisitedStack[StackPtr] = 2;

					StackPtr++;
					NodeIndexStack[StackPtr] = Op.RightIndex;
					ChildrenVisitedStack[StackPtr] = 0;

					const float2 BranchOffset = GetBranchOffset(Op, false);
					const float2 BranchOffsetNorm = normalize(BranchOffset);
					PrintPosStack[StackPtr] = ParentPrintPos + BranchOffset;
					AddLineSS(Context, ParentPrintPos + TREE_NODE_RADIUS * BranchOffsetNorm, PrintPosStack[StackPtr] - TREE_NODE_RADIUS * BranchOffsetNorm, TREE_COLOR);
				}
				else
				{
					StackPtr--;
				}
			}
			else
			{
				StackPtr--;
			}
		}

	#endif // SUBSTRATE_COMPILER

	}
#endif
}

#endif // SHADER_DEBUGSUBSTRATETREE_CS

#if SHADER_DEBUGSUBSTRATETREE_PS

#include "../DeferredShadingCommon.ush"
#define SUPPORT_CONTACT_SHADOWS 0
#include "../DeferredLightingCommon.ush"

#include "SubstrateEvaluation.ush"

// These defines are needed for environment lighting in SubstrateLightingCommon
#define ENABLE_DYNAMIC_SKY_LIGHT 1
#define ENABLE_SKY_LIGHT 1
#define SUPPORT_DFAO_INDIRECT_OCCLUSION 0
#define SUBSTRATE_FASTPATH 0
#define REFLECTION_COMPOSITE_USE_BLENDED_REFLECTION_CAPTURES 0
#include "../ReflectionEnvironmentShared.ush"
#include "../SkyLightingDiffuseShared.ush"
#include "../ReflectionEnvironmentComposite.ush"

#define USE_SUBSTRATE_ENV_LIGHTING_COMMON 1
#define USE_SUBSTRATE_FORWARD_LIGHTING_COMMON 1
#include "SubstrateLightingCommon.ush"

float4 GetBackgroundCheckboardColor(float2 UV)
{
	const float Count = 16.0f;
	uint2 UVi = uint2(UV * Count);
	bool bIsBlack = (UVi.x % 2) > 0 ? true : false;
	bIsBlack = (UVi.y % 2) > 0 ? !bIsBlack : bIsBlack;
	const float3 Color = (bIsBlack ? 0.04 : 0.18);
	return float4(Color, 0.0);
}

// Updated from http://jcgt.org/published/0007/03/04/
bool slabs(float3 p0, float3 p1, float3 rayOrigin, float3 invRaydir, out float outTMin, out float outTMax)
{
	float3 t0 = (p0 - rayOrigin) * invRaydir;
	float3 t1 = (p1 - rayOrigin) * invRaydir;
	float3 tmin = min(t0, t1), tmax = max(t0, t1);
	float maxtmin = max(max(tmin.x, tmin.y), tmin.z);
	float mintmax = min(min(tmax.x, tmax.y), tmax.z);
	outTMin = maxtmin;
	outTMax = mintmax;
	return maxtmin <= mintmax;
}

void EvaluateLighting(
	in FSubstrateBSDF BSDF,
	in float3 V,
	in float3x3 TangentBasis,
	in float ThicknessRatio,
	inout float4 OutColor)
{
	float3 TranslatedWorldPosition = PrimaryView.TranslatedWorldCameraOrigin;
	float SceneDepth = 0.0;

	const float3 N = TangentBasis[2];

	if (BSDF_GETSSSTYPE(BSDF) == SSS_TYPE_SIMPLEVOLUME)
	{
		// Apply transmittance on the background

		float3 DiffuseColor = SLAB_DIFFUSEALBEDO(BSDF);
		FParticipatingMedia PM = SubstrateSlabCreateParticipatingMedia(DiffuseColor, SLAB_SSSMFP(BSDF));
		const float3 Transmittance = IsotropicMediumSlabTransmittance(PM, ThicknessRatio * SUBSTRATE_SIMPLEVOLUME_THICKNESS_M, 1.f);

		OutColor.rgb *= Transmittance * BSDF.Coverage + (1.0f - BSDF.Coverage);
	}
	else
	{
		OutColor.rgb *= 0.0f;
	}

	// Patch the BSDF to appear as a top layer
	BSDF.LuminanceWeightV = 1.0f;
	BSDF.TransmittanceAboveAlongN = 1.0f;
	BSDF.bIsBottom = true;
	BSDF.bIsTop = true;
	BSDF.Coverage = 1.0f;

	// Create a head that will map to a single BSDF
	FSubstratePixelHeader SubstratePixelHeader = InitialiseSubstratePixelHeader();
	SubstratePixelHeader.ClosureCount = 1;

	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();
	FSubstrateAddressing NullSubstrateAddressing = (FSubstrateAddressing)0;

	// Apply emissive
	OutColor.rgb += BSDF_GETEMISSIVE(BSDF) * View.PreExposure;

	// Apply a white directional light
	if (ForwardLightStruct.HasDirectionalLight)
	{
		FDeferredLightData DirLightData = (FDeferredLightData)0;
		DirLightData.Color = ForwardLightStruct.DirectionalLightColor;
		DirLightData.FalloffExponent = 0;
		DirLightData.Direction = ForwardLightStruct.DirectionalLightDirection;
		DirLightData.DistanceFadeMAD = ForwardLightStruct.DirectionalLightDistanceFadeMAD;
		DirLightData.bRadialLight = false;
		DirLightData.SpecularScale = ForwardLightStruct.DirectionalLightSpecularScale;
		DirLightData.DiffuseScale = ForwardLightStruct.DirectionalLightDiffuseScale;
		float4 DirLightAttenuation = float4(1, 1, 1, 1);
		DirLightData.ShadowedBits = 1;
		DirLightData.ShadowMapChannelMask.x = 1;
		DirLightData.HairTransmittance = InitHairTransmittanceData();

		float Dither = 0.0f;
		float3 ToLight = DirLightData.Direction;
		float LightMask = 1.0f;
		FRectTexture RectTexture = InitRectTexture();
		uint LightChannelMask = 0xFFFFFFFF;
		uint PrimitiveLightingChannelMask = LightChannelMask;
		half4 PrecomputedShadowFactors = 1;

		float3 BSDFColoredVisibility = 1.0f;

		FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(TangentBasis, BSDF, V, ToLight);

		FSubstrateEvaluateResult BSDFEvaluate = (FSubstrateEvaluateResult)0;
		float3 DirLightLuminance = SubstrateForwardLightingCommon(
			Dither,
			Settings,
			DirLightData,
			ToLight,
			LightMask,
			DirLightAttenuation,
			RectTexture,
			LightChannelMask,
			PrimitiveLightingChannelMask,
			PrecomputedShadowFactors,
			TranslatedWorldPosition,
			SceneDepth,
			BSDFColoredVisibility,
			SubstratePixelHeader,
			SubstrateBSDFContext,
			BSDFEvaluate);

		OutColor.rgb += DirLightLuminance * View.PreExposure;
	}

	// Apply the sky box
	{
		float3 ToLight = V;
		FSubstrateBSDFContext SubstrateBSDFContext = SubstrateCreateBSDFContext(TangentBasis, BSDF, V, ToLight);

		const bool bEnableSpecular = ReflectionStruct.SkyLightParameters.y > 0.0f;
		FSubstrateEnvLightResult SubstrateEnvLight = SubstrateEvaluateForEnvLight(SubstrateBSDFContext, bEnableSpecular, Settings);

		FSubstrateIrradianceAndOcclusion SubstrateIrradianceAndOcclusion = SubstrateGetIrradianceAndAO(SubstratePixelHeader);

		float3 DiffuseLighting = 0.0;
		float3 SpecularLighting = 0.0;
		SubstrateEnvLightingCommon(
			SubstrateEnvLight,
			SubstrateIrradianceAndOcclusion,
			SubstrateBSDFContext,
			BSDF,
			N,	//in float3 BentNormal,
			1.0,//in float3 BSDFThroughput,
			0,	//in uint CaptureDataStartIndex,
			0,	//in uint NumCulledReflectionCaptures,
			1.0,//in float ScreenAmbientOcclusion,
			1.0,//in float CloudVolumetricAOShadow,
			1.0,//in float TopLayerSpecularContributionFactor,
			TranslatedWorldPosition,
			1.0,//in float CombinedScreenAndMaterialAO,
			DiffuseLighting,
			SpecularLighting);

		OutColor.rgb += (DiffuseLighting + SpecularLighting) * View.PreExposure;;
	}
}

void GetMaterialOnCube(
	in float4 SVPos,
	in float2 Pos,
	in float  Size,
	in FSubstrateBSDF BSDF,
	inout float4 OutColor)
{
	const float2 PixelPos = SVPos.xy;

	if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && all(PixelPos > Pos) && all(PixelPos < (Pos + Size)))
	{
		const float2 UV = (PixelPos - Pos) / Size;
		OutColor = GetBackgroundCheckboardColor(UV);

		const float3 RayO = -View.ViewForward*1.2;
		const float3 RayD = View.ViewForward + 1.0 * (UV.x - 0.5) * View.ViewRight - 1.0 * (UV.y - 0.5) * View.ViewUp;
		float3 V = -RayD;

		float2 Hit;
		const float CubeSize = 0.25;
		bool bIntersection = slabs(-CubeSize, CubeSize, RayO, 1/RayD, Hit.x, Hit.y);
		if (bIntersection)
		{
			float3 P = RayO + RayD * Hit.x;
			float3 N = normalize(P);
			float3 AbsN = abs(N);
			N = AbsN.x > AbsN.y && AbsN.x > AbsN.z ? float3(1 * sign(N.x), 0, 0) : N;
			N = AbsN.y > AbsN.z && AbsN.y > AbsN.x ? float3(0, 1 * sign(N.y), 0) : N;
			N = AbsN.z > AbsN.x && AbsN.z > AbsN.y ? float3(0, 0, 1 * sign(N.z)) : N;
			const float ThicknessRatio = abs(Hit.y - Hit.x) / CubeSize;

			float3x3 TangentBasis;
			TangentBasis[2] = N;
			TangentBasis[0] = AbsN.x > AbsN.y && AbsN.x > AbsN.z ? float3(0, 1, 0) : (AbsN.y > AbsN.z && AbsN.y > AbsN.x ? float3(0, 0, 1) : float3(1, 0, 0));
			TangentBasis[1] = AbsN.x > AbsN.y && AbsN.x > AbsN.z ? float3(0, 0, 1) : (AbsN.y > AbsN.z && AbsN.y > AbsN.x ? float3(1, 0, 0) : float3(0, 1, 0));

			EvaluateLighting(
				BSDF,
				V,
				TangentBasis,
				ThicknessRatio,
				OutColor);
		}
	}
}

void GetMaterialOnSphere(
	in float4 SVPos,
	in float2 Pos,
	in float  Size,
	in FSubstrateBSDF BSDF,
	inout float4 OutColor)
{
	const float2 PixelPos = SVPos.xy;

	if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && all(PixelPos > Pos) && all(PixelPos < (Pos + Size)))
	{
		const float2 UV = (PixelPos - Pos) / Size;
		OutColor = GetBackgroundCheckboardColor(UV);

		float3 V = -View.ViewForward;
		const float3 RayO = V + (UV.x-0.5) * View.ViewRight + -(UV.y-0.5) * View.ViewUp;
		const float3 RayD = -V;
		const float4 Sphere = float4(0.0f, 0.0f, 0.f, 0.48);
		const float2 Hit = RayIntersectSphere(RayO, RayD, Sphere);
		if (Hit.x >= 0)
		{
			float3 P = RayO + RayD * Hit.x;
			float3 N = normalize(P - Sphere.xyz);
			const float ThicknessRatio = (Hit.y - Hit.x) / Sphere.w;

			float3x3 TangentBasis;
			TangentBasis[2] = N;
			TangentBasis[0] = normalize(N * float3(1, 0, 0));
			TangentBasis[1] = cross(N, TangentBasis[0]);
			TangentBasis[0] = -cross(N, TangentBasis[1]);

			EvaluateLighting(
				BSDF,
				V,
				TangentBasis,
				ThicknessRatio,
				OutColor);
		}
	}
}

void GetMaterialSlice(
	in float4 SVPos,
	in float2 Pos,
	in float2  Size,
	in FSubstrateBSDF BSDF,
	inout float4 OutColor)
{
	const float2 PixelPos = SVPos.xy;

	if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && all(int2(PixelPos) > int2(Pos)) && all(int2(PixelPos) < int2(Pos + Size)))
	{
		const float2 UV = (PixelPos - Pos) / Size;
		OutColor = GetBackgroundCheckboardColor((PixelPos - Pos) / Size.x);

		// View and normal along up.
		float3 V = float3(0, 0, 1);
		float3 N = float3(0, 0, 1);
		const float ThicknessRatio = 1.0;

		float3x3 TangentBasis;
		TangentBasis[2] = N;
		TangentBasis[0] = float3(1, 0, 0);
		TangentBasis[1] = float3(0, 1, 0);

		EvaluateLighting(
			BSDF,
			V,
			TangentBasis,
			ThicknessRatio,
			OutColor);
	}
	else
	{
		OutColor = 0;
	}
}

void MaterialDebugSubstrateTreePS(
	float4 SVPos : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	OutColor = float4(0.0f, 0.0f, 0.0f, 1.0f);

	// Workaround on Vulkan where this shader does not compile correctly
#if !VULKAN_PROFILE & !VULKAN_PROFILE_SM5 && !VULKAN_PROFILE_SM6 && !COMPILER_FXC

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset((float2(GetCursorPosition()) * View.ViewResolutionFraction), uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	if (Header.ClosureCount > 0)
	{
		const float2 PixelPos = SVPos.xy;
		OutColor = float4(0.0f, 0.0f, 0.0f, PixelPos.x < 540.0f ? 0.5f : 1.0f); // Background
		if (int(PixelPos.x) == 280 && int(PixelPos.y) > 90)
		{
			OutColor = float4(1.0f, 1.0f, 1.0f, 0.5f);							// Operator/BSDF separator
		}

		const int SliceStoringDebugSubstrateTreeData = Substrate.SliceStoringDebugSubstrateTreeData;

		FSubstrateTreeHeader SubstrateTreeHeader = (FSubstrateTreeHeader)0;
		SubstrateUnpackInSubstrateTreeHeader(
			Substrate.MaterialTextureArray,
			SubstrateTreeHeader,
			SliceStoringDebugSubstrateTreeData);


		float2 Pos = float2(BSDF_ELEMENT_STARTX, BSDF_ELEMENT_STARTY);
		const float MaterialPanelSize = 200.0f;

		// SUBSTRATE_TODO: read BSDF and Operators "on the fly".
		{
			SubstrateTreeHeader.BSDFCount = min(SubstrateTreeHeader.BSDFCount, 2);
			SUBSTRATE_UNROLL_N(2)
			for (int i = 0; i < SubstrateTreeHeader.BSDFCount; i++)
			{
				FSubstrateBSDF BSDF = SubstrateUnpackInSubstrateTreeBSDF(
					i,
					Substrate.MaterialTextureArray,
					SubstrateTreeHeader,
					SliceStoringDebugSubstrateTreeData);

				//if (BSDF.Coverage > 0.0f)
				{
					GetMaterialOnSphere(
						SVPos,
						Pos,
						MaterialPanelSize,
						BSDF,
						OutColor);

					GetMaterialOnCube(
						SVPos,
						Pos + float2(MaterialPanelSize + 10.0f, 0.0f),
						MaterialPanelSize,
						BSDF,
						OutColor);

					Pos.y += MaterialPanelSize + 10.0f;
				}
			}
		}

		#define MAX_LAYER_COUNT 4
		float LayerAccCoverage[MAX_LAYER_COUNT];
		{
			for (int i = 0; i < MAX_LAYER_COUNT; ++i)
			{
				LayerAccCoverage[i] = 0.0f;
			}
		}

		float2 MatTopOrigin = float2(BSDF_ELEMENT_STARTX + 450, BSDF_ELEMENT_STARTY);// float2(BSDF_ELEMENT_STARTX, 550);
		float2 MatTopLayerSize = float2(410, 100);
		
		// This is a really basic material topology debug. SUBSTRATE_TODO would be to draw the material graph itself with anottation
		{
			SubstrateTreeHeader.BSDFCount = min(SubstrateTreeHeader.BSDFCount, 2);
			SUBSTRATE_UNROLL_N(2)
			for (int i = 0; i < SubstrateTreeHeader.BSDFCount; i++)
			{
				FSubstrateBSDF BSDF = SubstrateUnpackInSubstrateTreeBSDF(
					i,
					Substrate.MaterialTextureArray,
					SubstrateTreeHeader,
					SliceStoringDebugSubstrateTreeData);

				FSubstrateOperator Op = SubstrateUnpackInSubstrateTreeOperator(
					BSDF.OperatorIndex,
					Substrate.MaterialTextureArray,
					SubstrateTreeHeader,
					SliceStoringDebugSubstrateTreeData);

				const uint LayerDepth = Op.LayerDepth;

				if (LayerDepth < MAX_LAYER_COUNT)
				{
					// Draw the slab information
					float2 LayerOrigin = MatTopOrigin + MatTopLayerSize.xy * float2(LayerAccCoverage[LayerDepth], LayerDepth);
					float2 SlabSize = MatTopLayerSize.xy * float2(BSDF.Coverage, 1.0);

					if (BSDF_GETTYPE(BSDF) == SUBSTRATE_BSDF_TYPE_SLAB && all(int2(PixelPos) >= int2(LayerOrigin)) && all(int2(PixelPos) <= int2(LayerOrigin + SlabSize)))
					{
						GetMaterialSlice(
							SVPos,
							LayerOrigin,
							SlabSize,
							BSDF,
							OutColor);
					}

					LayerAccCoverage[LayerDepth] += BSDF.Coverage;
				}
			}
		}
	}

	OutColor = float4(pow(OutColor.rgb, 1.0 / 2.2), OutColor.a);
#endif
}

#endif // SHADER_DEBUGSUBSTRATETREE_PS


///////////////////////////////////////////////////////////////////////////////////////////////////
// Material Print

#if SHADER_SYSTEMINFO

Buffer<uint> ClassificationTileDrawIndirectBuffer;

uint GetTileCount(uint InType)
{
	return ClassificationTileDrawIndirectBuffer[InType * 4 + 1];
}

[numthreads(1, 1, 1)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	if (all(DispatchThreadId == 0))
	{
		ConvertToSerializableSubstratePixelDebugData();
	}
}
#endif // SHADER_SYSTEMINFO