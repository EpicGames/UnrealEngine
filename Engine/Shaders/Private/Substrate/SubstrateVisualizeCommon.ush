// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

#if SUBSTRATE_ENABLED

#include "/Engine/Private/ShaderPrint.ush"
#include "/Engine/Private/Substrate/SubstrateRead.ush"

#ifndef SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
#error SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE needs to be defined
#endif

void AddTipLineTWS(inout FShaderPrintContext Ctx, float3 TWSPos, float3 In, float4 InTipColor, float4 InCoreColor=ColorYellow, float Scale=1.0f)
{
	const float CoreSize = 0.8f * Scale;
	const float TipSize  = 1.0f * Scale;
	AddLineTWS(Ctx, TWSPos,                 TWSPos + In * CoreSize, InCoreColor, InCoreColor);
	AddLineTWS(Ctx, TWSPos + In * CoreSize, TWSPos + In * TipSize,  InTipColor,  InTipColor);
}

void DrawReferentialTWS(inout FShaderPrintContext Ctx, float3 P /*In Translated World Space*/, float3 X, float3 Y, float3 N, float4 InColor)
{
	const float Size = 10.f;
	AddTipLineTWS(Ctx, P, X, ColorRed, InColor, Size);
	AddTipLineTWS(Ctx, P, Y, ColorBlue, InColor, Size);
	AddTipLineTWS(Ctx, P, N, ColorGreen, InColor, Size);
}

void AddDrawPixelFootprint(float3 P, float3 dPdx, float3 dPdy, float2 Scale, bool bNormalize, float4 Color)
{
	const float3 T = (bNormalize ? normalize(dPdx) : dPdx) * Scale.x;
	const float3 B = (bNormalize ? normalize(dPdy) : dPdy) * Scale.y;
	const float3 N = normalize(cross(T, B));

	const float3 WP0 = P - T - B;
	const float3 WP1 = P + T - B;
	const float3 WP2 = P + T + B;
	const float3 WP3 = P - T + B;

	AddLineTWS(WP0, WP1, Color);
	AddLineTWS(WP1, WP2, Color);
	AddLineTWS(WP2, WP3, Color);
	AddLineTWS(WP3, WP0, Color);
}

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
void DrawPixelFootprint(float3 P, float3 dPdx, float3 dPdy, uint2 PixelCoord)
{
	const FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelCoord);
	const bool bIsValid = SubstrateSubSurfaceHeaderGetIsValid(SSSHeader);
	const bool bIsProfile = SubstrateSubSurfaceHeaderGetIsProfile(SSSHeader);

	if (bIsValid)
	{
		float3 MFP = 0;
		if (bIsProfile)
		{
			MFP = GetSubsurfaceProfileMFPInCm(SubstrateSubSurfaceHeaderGetProfileId(SSSHeader)).xyz * SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
		}
		else if(SSSHEADER_TYPE(SSSHeader) == SSS_TYPE_TRANSMITTANCE_SHADOW)
		{
			MFP = SubstrateSubSurfaceHeaderGetTransmittanceShadowMFPInMeters(SSSHeader);
		}
		else
		{
			MFP = SubstrateSubSurfaceHeaderGetMFP(SSSHeader);
		}

		FSubstratePixelFootprint Footprint = SubstrateGetPixelFootprint(dPdx, dPdy, 0.f /*InNormalCurvatureRoughness*/);
		AddDrawPixelFootprint(P, dPdx, dPdy, 0.5f, false, ColorRed);
		AddDrawPixelFootprint(P, dPdx, dPdy, Footprint.PixelRadiusInWorldSpace, true, ColorOrange);
		AddDrawPixelFootprint(P, dPdx, dPdy, max3(MFP.x, MFP.y, MFP.z), true, ColorCyan);
	}
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Material Print

void PrintPixelType(inout FShaderPrintContext Ctx, in FSubstratePixelHeader Header, FFontColor InColor)
{
	if (Header.IsSimpleMaterial())						Print(Ctx, TEXT("Simple  "), InColor);
	else if (Header.IsSingleMaterial())					Print(Ctx, TEXT("Single  "), InColor);
	else if (Header.IsComplexSpecialMaterial())			Print(Ctx, TEXT("Complex Special "), InColor);
	else if (Header.IsComplexMaterial())				Print(Ctx, TEXT("Complex "), InColor);
	else if (Header.IsSingleLayerWater())				Print(Ctx, TEXT("SLWater "), InColor);
	else if (Header.IsHair())							Print(Ctx, TEXT("Hair    "), InColor);
	else if (Header.IsEye())							Print(Ctx, TEXT("Eye     "), InColor);
	else												Print(Ctx, TEXT("Error - Unkown pixel type"), FontRed);
}

void PrintSSSType(inout FShaderPrintContext Ctx, uint SSSType, bool bThin, FFontColor InColor)
{
		 if (SSSType == SSS_TYPE_TWO_SIDED_WRAP)	Print(Ctx, TEXT("Two-Sided Wrap"), InColor);
	else if (SSSType == SSS_TYPE_WRAP)				Print(Ctx, TEXT("Wrap"), InColor);
	else if (SSSType == SSS_TYPE_DIFFUSION)			Print(Ctx, TEXT("Diffusion"), InColor);
	else if (SSSType == SSS_TYPE_DIFFUSION_PROFILE)	Print(Ctx, TEXT("Diffusion Profile"), InColor);
	else if (SSSType == SSS_TYPE_SIMPLEVOLUME)		Print(Ctx, TEXT("Simple Volume"), InColor);
	else if (bThin)									Print(Ctx, TEXT("Thin"), InColor);
	else /*	(SSSType == SSS_TYPE_NONE) */			Print(Ctx, TEXT("None"), InColor);
}

void PrintBackgroundRect(inout FShaderPrintContext Ctx, float2 StartRect, float2 EndRect)
{
	const float4 RectBackgroundColor = float4(1, 1, 1, 0.25f);
	AddFilledQuadSS(
		(StartRect - ShaderPrintData.FontSize) * ShaderPrintData.Resolution,
		(EndRect + ShaderPrintData.FontSize) * ShaderPrintData.Resolution,
		RectBackgroundColor);
}

void PrintAddress(inout FShaderPrintContext Context, inout FSubstrateAddressing SubstrateAddressing)
{
	FFontColor Font;
	Font.Color = lerp(FontEmerald.Color, FontSilver.Color, 0.75f);
	Print(Context, TEXT("[Address="), Font);
	Print(Context, SubstrateAddressing.CurrentIndex, Font, 2, 0);
	Print(Context, TEXT("]"), Font);
}

void SubstratePrintBSDF(
	inout FShaderPrintContext Context, 
	FSubstrateBSDFContext BSDFContext,
	float3 WorldPosition, 
	float3 V)
{
	const float LineSize = 5.f;
	const float3 WorldNormal = BSDFContext.N;

	// Draw Referential
	if (BSDF_GETTYPE(BSDFContext.BSDF) != SUBSTRATE_BSDF_TYPE_EYE)
	{
		DrawReferentialTWS(Context, WorldPosition, BSDFContext.X, BSDFContext.Y, BSDFContext.N, ColorYellow);
	}

	switch (BSDF_GETTYPE(BSDFContext.BSDF))
	{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			const bool bHasHaziness = BSDF_GETHASHAZINESS(BSDFContext.BSDF);
			if (bHasHaziness)
			{
				FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDFContext.BSDF));
				const bool  bHazeAsSimpleClearCoat = Haziness.bSimpleClearCoat;

			#if CLEAR_COAT_BOTTOM_NORMAL
				if (Haziness.HasBottomNormal())
				{
					const float3 BottomNormal = SubstrateGetSimpleCoatBottomNormal(Haziness.BottomNormalOct, WorldNormal);
					AddTipLineTWS(Context, WorldPosition, BottomNormal, ColorPurple, ColorYellow, 10.f);
				}
			#endif
			}
		}
		break;

		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
		}
		break;

		case SUBSTRATE_BSDF_TYPE_EYE:
		{
			// Tangent basis for the iris
			const float EyeLineSize = 1.f;
			AddTipLineTWS(Context, WorldPosition, BSDFContext.N, ColorBlue, ColorYellow, EyeLineSize);
			AddTipLineTWS(Context, WorldPosition, EYE_IRISNORMAL(BSDFContext.BSDF), ColorRed, ColorYellow, EyeLineSize);
			AddTipLineTWS(Context, WorldPosition, EYE_IRISPLANENORMAL(BSDFContext.BSDF), ColorGreen, ColorYellow, EyeLineSize);

		}
		break;

		case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		{
		}
		break;

		case SUBSTRATE_BSDF_TYPE_TOON:
		{
		}
		break;

	default:
		{
			Print(Context, TEXT("Error - Uknown BSDF type"), FontRed);
		}
		break;
	}
}

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 1
void SubstratePrintMaterialProperties(inout FShaderPrintContext Context, uint2 InCoord, float3 InWorldPosition, float3 V, uint InClosureIndex)
{
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(InCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);

	// BSDFs
	if (InClosureIndex < Header.ClosureCount)
	{
		#if SUBSTRATE_MATERIAL_CLOSURE_COUNT > 1
		if (InClosureIndex > 0)
		{
			const uint AddressOffset = UnpackClosureOffsetAtIndex(Substrate.ClosureOffsetTexture[InCoord], InClosureIndex, Header.ClosureCount);
			SubstrateSeekClosure(SubstrateAddressing, AddressOffset);
		}
		#endif

		const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Substrate.MaterialTextureArray, SubstrateAddressing, Header);
		FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(Header, BSDF, SubstrateAddressing, V, float3(0, 0, 1) /*DummyL*/);
		SubstratePrintBSDF(Context, BSDFContext, InWorldPosition, V);
	}
}
#elif SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 0
void SubstratePrintMaterialProperties(inout FShaderPrintContext Context, uint2 InCoord, float3 InWorldPosition, float3 V, uint InClosureIndex)
{
	// BSDFs
	if (InClosureIndex < 1)
	{
		FScreenSpaceData ScreenSpaceData = GetScreenSpaceDataUint(InCoord);
		FSubstrateGBufferBSDF GBufferBSDF = SubstrateReadGBufferBSDF(ScreenSpaceData);
		FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(GBufferBSDF.BSDF, GBufferBSDF.BSDFTangentBasis, V, float3(0, 0, 1) /*DummyL*/);
		SubstratePrintBSDF(Context, BSDFContext, InWorldPosition, V);
	}
}
#else // Raytracing payload
void SubstratePrintMaterialProperties(inout FShaderPrintContext Context, uint2 InCoord, float3 InWorldPosition, float3 V, uint InClosureIndex, FSubstrateRaytracingPayload Payload)
{
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(InCoord, uint2(View.BufferSizeAndInvSize.xy), 0);		
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Payload, SubstrateAddressing, Payload);

	for (uint ClosureIndex=0;ClosureIndex<Header.ClosureCount;ClosureIndex++)
	{
		const FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Payload, SubstrateAddressing, Header); // Take the parameter of the first BSDF
		const float3x3 TangentBasis = SubstrateGetBSDFSharedBasis(Header, BSDF, SubstrateAddressing);
		FSubstrateBSDFContext BSDFContext = SubstrateCreateBSDFContext(TangentBasis, BSDF, V, float3(0, 0, 1) /*DummyL*/, false/*bHasValidL*/);
		SubstratePrintBSDF(Context, BSDFContext, InWorldPosition, V);
	}
}
#endif

#endif // SUBSTRATE_ENABLED