// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SubsurfaceBurleyNormalized.ush: Screenspace Burley subsurface scattering implementation.
=============================================================================*/
#pragma once

#include "Random.ush"
#include "DeferredShadingCommon.ush"
#include "MonteCarlo.ush"
#include "Substrate/Substrate.ush"

// Setup the max number of Burley samples
#define BURLEY_NUM_SAMPLES	64
#define BURLEY_INV_NUM_SAMPLES (1.0f/BURLEY_NUM_SAMPLES)

#define EXPONENTIAL_WEIGHT 0.2f

// Set to 1 to be more correct and higher quality. we estimate true distribution variance. It would be slower under some conditions
// Set to 0 to be more efficient. That we use the previous sample count as the mean.
// By default we set it to 0. Enable it for half resolution.
#define USE_TRUE_DISTRIBUTION_VAR (SUBSURFACE_HALF_RES)

// Used to avoid low sampling count due to low variance
#define BETA_LIMIT 8

// Use Bilateral filtering or not
#define USE_BILATERAL_FILTERING 1

#define RADIUS_SAMPLE_UNIFORM_DISK 0

// Miplevel constant parameter, the parameter is determined to have algorithm perform  output the best quality and speed without introducing artifacts.
// It is used to reduce the effect of the number of samples on miplevel.
#define MIP_CONSTANT_FACTOR 0.0625f

// use point sampler for LDS
#ifdef SUBSURFACE_SAMPLER_TYPE
#undef SUBSURFACE_SAMPLER_TYPE
#define SUBSURFACE_SAMPLER_TYPE 0 
#endif

// quality/performance options

#define RESAMPLE_PDF 0
#define REPROJECTION 1

// one of these must be true
#define ROOT_APROXIMATE 1
#define ROOT_FINDING 0
#define ROOT_ANALYTIC 0

// one of these must be true
#define SAMPLE_ROOT_ANGLE_R2SEQUENCE 1
#define SAMPLE_ANGLE_RANDOM 0
#define SAMPLE_ANGLE_FIBONACCI 0

// Texture local cash does not help on 2080TI with regular layout
#define TEXTURE_CACHE_DISABLED 0

#define MORTON_USE_LUT 0
#define REWEIGHT_CENTER_SAMPLE 1

#define VARIANCE_LEVEL 0.0001
#define HIGH_LUMA_SAMPLE_COUNT 8
#define LOW_LUMA_SAMPLE_COUNT 16
#define PROFILE_EDGE_SAMPLE_COUNT 32

// for any undefined optiones, define them to 0
#ifndef RESAMPLE_PDF
#define RESAMPLE_PDF 0
#endif

#ifndef REPROJECTION
#define REPROJECTION 0
#endif

#ifndef ENABLE_VELOCITY
#define ENABLE_VELOCITY 0
#endif

#ifndef SUBSURFACE_BURLEY_COMPUTE
#define SUBSURFACE_BURLEY_COMPUTE 0
#endif

#ifndef SUBSURFACE_SINGLE_PASS
#define SUBSURFACE_SINGLE_PASS 0
#endif

#ifndef ROOT_APROXIMATE
#define ROOT_APROXIMATE 0
#endif

#ifndef ROOT_FINDING
#define ROOT_FINDING 0
#endif

#ifndef ROOT_ANALYTIC
#define ROOT_ANALYTIC 0
#endif

#ifndef SAMPLE_ROOT_ANGLE_R2SEQUENCE
#define SAMPLE_ROOT_ANGLE_R2SEQUENCE 0
#endif

#ifndef SAMPLE_ANGLE_RANDOM
#define SAMPLE_ANGLE_RANDOM 0
#endif

#ifndef SAMPLE_ANGLE_FIBONACCI
#define SAMPLE_ANGLE_FIBONACCI 0
#endif

#ifndef TEXTURE_CACHE_DISABLED
#define TEXTURE_CACHE_DISABLED 0
#endif

#ifndef REWEIGHT_CENTER_SAMPLE
#define REWEIGHT_CENTER_SAMPLE 1
#endif

#ifndef MORTON_USE_LUT
#define MORTON_USE_LUT 0
#endif

#ifndef ENABLE_PROFILE_ID_CACHE
#define ENABLE_PROFILE_ID_CACHE 0
#endif

#define BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH				0
#define BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL	1

#ifndef BILATERAL_FILTER_KERNEL_FUNCTION_TYPE
#define BILATERAL_FILTER_KERNEL_FUNCTION_TYPE BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
#endif

#ifndef SUBSURFACE_SAMPLER_TYPE
#define SUBSURFACE_SAMPLER_TYPE 0
#endif

float SampleDepthTexture(float2 ScreenUV)
{
#if SUBSURFACE_HALF_RES && !(SUBSURFACE_SINGLE_PASS)
	ConvertToDeviceZ(Texture2DSample(SubsurfaceInput1_Texture, SubsurfaceSampler1, ScreenUV).g);
#else
	float2 FullScreenUV = ScreenUV;
#endif
	
	return Texture2DSampleLevel(SceneTexturesStruct.SceneDepthTexture, SceneTexturesStruct_SceneDepthTextureSampler, ScreenUV, 0).r;
}

float2 Generate2DRandomNumber(int3 Seed)
{
#if SAMPLE_ROOT_ANGLE_R2SEQUENCE
	return R2Sequence(Seed.z);
#else
	return float2(Rand3DPCG16(Seed).xy) / 0x10000;
#endif
}

struct FBurleySampleInfo
{
	float RadiusInMM;
	float Theta;
	float Pdf;
	float CosTheta;
	float SinTheta;
};

#define FIBONACCI_SEQUENCE_ANGLE(x) (((float(x) + 0.5)*(1 + sqrt(5))*0.5) * 2 * PI)

// angle, cosine, and sine
#define FIBONACCI_SEQUENCE_TRIPLE(x) { FIBONACCI_SEQUENCE_ANGLE(x), cos(FIBONACCI_SEQUENCE_ANGLE(x)), sin(FIBONACCI_SEQUENCE_ANGLE(x)) }

FBurleySampleInfo GenerateSampleInfo(float2 Rand0T1, float DiffuseMeanFreePathForSample, float SpectralForSample, uint SequenceId)
{
	FBurleySampleInfo BurleySampleInfo;

	// Direct sampling of angle is more efficient and fast in test when the dmfp is small.
	// However, FIB has better quality when dmfp and world unit scale is large.

	// Sample radius
#if ROOT_ANALYTIC
	// clever analytical solution
	float FoundRoot = RadiusRootFindAnalytic(DiffuseMeanFreePathForSample / SpectralForSample, Rand0T1.x);
#elif ROOT_FINDING
	// root finding using derivatives
	float FoundRoot = RadiusRootFinding(DiffuseMeanFreePathForSample / SpectralForSample, Rand0T1.x, DiffuseMeanFreePathForSample);
#elif ROOT_APROXIMATE
	//Approximation
	float FoundRoot = RadiusRootFindByApproximation(DiffuseMeanFreePathForSample / SpectralForSample, Rand0T1.x);
#endif

	BurleySampleInfo.RadiusInMM = max(FoundRoot, 0.00001f);
	
	// Sample angle
#if (SAMPLE_ANGLE_RANDOM || SAMPLE_ROOT_ANGLE_R2SEQUENCE)
	BurleySampleInfo.Theta = Rand0T1.y * 2 * PI;

	BurleySampleInfo.CosTheta = cos(BurleySampleInfo.Theta);
	BurleySampleInfo.SinTheta = sin(BurleySampleInfo.Theta);
#elif SAMPLE_ANGLE_FIBONACCI
	// Fibonacci sequence for angle. Randoness is expensive for converging.

	BurleySampleInfo.Theta = FIBONACCI_SEQUENCE_ANGLE(SequenceId);
	BurleySampleInfo.CosTheta = cos(BurleySampleInfo.Theta);
	BurleySampleInfo.SinTheta = sin(BurleySampleInfo.Theta);

#endif

	// Estimate Pdf
	BurleySampleInfo.Pdf = GetPdf(BurleySampleInfo.RadiusInMM, DiffuseMeanFreePathForSample, SpectralForSample);

	return BurleySampleInfo;
}

#if SUBSTRATE_GBUFFER_FORMAT==1
FSubstrateTopLayerData LoadSubstrateTopLayerData(float2 UV)
{
	const float2 PixelPos = UV.xy * View.BufferSizeAndInvSize.xy;
	return SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelPos, 0)));
}
#endif


float GetProfileMask(float2 BufferUV)
{
#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(BufferUV);
	const bool bIsProfile = SubstrateSubSurfaceHeaderGetIsValid(SSSHeader);
#else
	const FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const bool bIsProfile = UseSubsurfaceProfile(ScreenSpaceData.GBuffer.ShadingModelID);	
#endif

	float Ret = 0;
	BRANCH if (bIsProfile)
	{
		Ret = 1.0f;
	}

	return Ret;
}

float GetProfileEdgeMask(float2 BufferUV)
{
#if SUBSURFACE_HALF_RES
	#define PIXELOFFSET_UVDELTA 0.5f
#else
	#define PIXELOFFSET_UVDELTA 1.0f
#endif

	float P11 = GetProfileMask((BufferUV + float2(0.0f, 0.0f) * SubsurfaceInput0_ExtentInverse));
	float P00 = GetProfileMask((BufferUV + float2(-PIXELOFFSET_UVDELTA, -PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));
	float P01 = GetProfileMask((BufferUV + float2(0.0f, -PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));
	float P02 = GetProfileMask((BufferUV + float2(PIXELOFFSET_UVDELTA, -PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));
	float P10 = GetProfileMask((BufferUV + float2(-PIXELOFFSET_UVDELTA, 0.0f) * SubsurfaceInput0_ExtentInverse));
	float P12 = GetProfileMask((BufferUV + float2(PIXELOFFSET_UVDELTA, 0.0f) * SubsurfaceInput0_ExtentInverse));
	float P20 = GetProfileMask((BufferUV + float2(-PIXELOFFSET_UVDELTA, PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));
	float P21 = GetProfileMask((BufferUV + float2(0.0f, PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));
	float P22 = GetProfileMask((BufferUV + float2(PIXELOFFSET_UVDELTA, PIXELOFFSET_UVDELTA) * SubsurfaceInput0_ExtentInverse));

	return (P00 + P01 + P02 + P10 + P11 + P12 + P20 + P21 + P22) / 9.0f;
}

float RadiusRootFindingCM(float D, float RandomNumber, float X0)
{
	return RadiusRootFinding(D*0.01, RandomNumber, X0*0.01)*100.0f;
}

float GetPdfInCM(float Radius, float L, float S)
{
	return GetPdf(Radius *0.01f, L*0.01f, S);
}

// Get the history states
// SubsurfaceInput2_Texture stores the encoded velocity
// SubsurfaceInput1_Texture store the history state
struct FHistoryState
{
	float4 History;
	bool OffScreen;
};

FHistoryState GetHistoryState(float2 BufferUV)
{
	FHistoryState HistoryState = (FHistoryState)0;
	float2 VelocityOffset = float2(0.0, 0.0);
	float2 NearestBufferUV = BufferUV;
	float2 ViewportUV = (BufferUV - SubsurfaceInput0_UVViewportMin.xy) * SubsurfaceInput0_UVViewportSizeInverse.xy;
	bool OffScreen = false;

	//	Get the history with reprojection
	//  Implement a simplified version of temporal AA
#if REPROJECTION

	// Code adapted from temporal AA
	float3 PosN;
	PosN.xy = ViewportUVToScreenPos(ViewportUV);
	PosN.z = SampleDepthTexture(BufferUV);// Direct sample without converting to world space;
	float4 ThisClip = float4(PosN.xy, PosN.z, 1.0f);
	float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
	float2 PrevScreen = PrevClip.xy / PrevClip.w;
	float2 BackN = PosN.xy - PrevScreen;

	// Sample the velocity texture
	float Velocity = 0;

	float2 BackTemp = BackN * SubsurfaceInput1_ViewportSize.xy;

#if (ENABLE_VELOCITY)
	{
		float4 EncodedVelocity = Texture2DSampleLevel(SubsurfaceInput2_Texture, SubsurfaceSampler2, NearestBufferUV + VelocityOffset, 0);

		if (EncodedVelocity.x > 0.0)
		{
			BackN = DecodeVelocityFromTexture(EncodedVelocity).xy;
		}

		BackTemp = BackN * SubsurfaceInput1_ViewportSize.xy;
	}
#endif

	// Update velocity
	Velocity = sqrt(dot(BackTemp, BackTemp));

	float2 HistoryScreenPosition = (PosN.xy - BackN);

	// Detect if HistoryBufferUV would be outside of the viewport.
	OffScreen = max(abs(HistoryScreenPosition.x), abs(HistoryScreenPosition.y)) >= 1.0;

	float4 History = 0;

	BRANCH if (!OffScreen)
	{
		// ScreenPos to bufferUV
		float2 HistoryUV = ScreenPosToViewportUV(HistoryScreenPosition);
		// Convert history uv in viewport to buffer uv
		float2 HistoryBufferUV = HistoryUV * SubsurfaceInput0_UVViewportSize.xy + SubsurfaceInput0_UVViewportMin.xy;
		History = Texture2DSample(SubsurfaceInput1_Texture, SubsurfaceSampler1, HistoryBufferUV);
	}

#else
	float4 History = Texture2DSample(SubsurfaceInput1_Texture, SubsurfaceSampler1, BufferUV);
#endif

	HistoryState.History = History;
	HistoryState.OffScreen = OffScreen;

	return HistoryState;
}

int GetNumOfSamplesBasedOnQuality(float2 UV)
{
	FHistoryState HistoryState = GetHistoryState(UV);
	float4 QualityMatrix = HistoryState.History;
	float VariableVar = QualityMatrix.b;

	float NumOfSamples = 0;
#if USE_TRUE_DISTRIBUTION_VAR
	float EstimatedCount = QualityMatrix.g * (2 /EXPONENTIAL_WEIGHT - 1);
	NumOfSamples = clamp((QualityMatrix.b * EstimatedCount / VARIANCE_LEVEL) - EstimatedCount, 8, BURLEY_NUM_SAMPLES);
#else
	//1. Estimate the number of samples required to reach the target variance level
	NumOfSamples = clamp((VariableVar / VARIANCE_LEVEL), BETA_LIMIT, BURLEY_NUM_SAMPLES);//View.GeneralPurposeTweak
#endif
	// The following two ad hoc design is not viable when we monitor the variance of control variates residual.
	//1.1 If the luminance is too low, we should increase several number of samples to oberse the world for high lighting condition
	// NumOfSamples = lerp(NumOfSamples, max(LOW_LUMA_SAMPLE_COUNT, NumOfSamples), step(0.5, QualityMatrix.a));

	//2. Clamp down the number of samples when the gamma corrected luminance is too large
	//   because it does not help to improve the quality, but degrade the performance.
	// NumOfSamples = lerp(NumOfSamples, HIGH_LUMA_SAMPLE_COUNT, step(3.2, QualityMatrix.a)); // 3.2 is an experimental value to have best quality.

	//3. Reduce the number of samples at the edge due to TAA (Flickering)
#if 0
	float Mask = Texture2DSample(SubsurfaceInput2_Texture, SharedSubsurfaceSampler2, UV).r;
	NumOfSamples = lerp(NumOfSamples, PROFILE_EDGE_SAMPLE_COUNT, step(Mask, 0.99));
#endif

	return NumOfSamples;
}

// If the shading model is valid, return the profile. Otherwise, return SSS_PROFILE_ID_INVALID.
// That way if we have a valid profile, we can check of the other profile is valid with a single boolean expression.
#if SUBSTRATE_GBUFFER_FORMAT==1
uint ExtractSubsurfaceProfileIntWithInvalid(FSubstrateSubsurfaceHeader SSSHeader)
{
	return SubstrateSubSurfaceHeaderGetProfileId(SSSHeader);
}
#else
uint ExtractSubsurfaceProfileIntWithInvalid(FGBufferData BufferData)
{
	uint ProfileID = SSS_PROFILE_ID_INVALID;
	if (UseSubsurfaceProfile(BufferData.ShadingModelID))
	{
		ProfileID = ExtractSubsurfaceProfileInt(BufferData);
	}
	return ProfileID;
}
#endif

#define BILATERAL_FILTER_DEFAULT_NORMAL float3(1.0f,0.0f,0.0f)

struct BurleySampleDiffuseNormal
{
	float4 DiffuseLighting;
	uint   ProfileID;

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	float3 WorldNormal;
#endif

};

#if SUBSURFACE_BURLEY_COMPUTE

// Configurations for each small thread
#define THREAD_SIZE_1D 8
#define THREAD_SIZE_X THREAD_SIZE_1D
#define THREAD_SIZE_Y THREAD_SIZE_X
#define THREAD_TOTAL_SZIE (THREAD_SIZE_X*THREAD_SIZE_Y)

#define THREAD_TEXTURE_BORDER 1
#define THREAD_TEXTURE_DIFFUSE_REGION_SIZE   THREAD_SIZE_X
#define THREAD_TEXTURE_DIMENSION_SIZE (THREAD_TEXTURE_DIFFUSE_REGION_SIZE + 2*THREAD_TEXTURE_BORDER)
#define THREADGROUP_TEXTURE_SHARE_TOTALSIZE (THREAD_TEXTURE_DIMENSION_SIZE*THREAD_TEXTURE_DIMENSION_SIZE)

// Configuration of group threads
#ifndef SUBSURFACE_GROUP_SIZE
#define SUBSURFACE_GROUP_SIZE 8
#endif

#define LARGE_GROUP_TOTAL_SIZE (SUBSURFACE_GROUP_SIZE*SUBSURFACE_GROUP_SIZE)
#define LARGE_GROUP_DIFFUSE_REGION_SIZE SUBSURFACE_GROUP_SIZE
#define LARGE_GROUP_TEXTURE_DIMENSION_SIZE (LARGE_GROUP_DIFFUSE_REGION_SIZE+2*THREAD_TEXTURE_BORDER)
#define LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE (LARGE_GROUP_TEXTURE_DIMENSION_SIZE*LARGE_GROUP_TEXTURE_DIMENSION_SIZE)

#define NUM_OF_PASS_REQUIRED_FILL_SHARED_DIFFUSE_TEXTURE ((LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE+THREAD_TOTAL_SZIE-1)/THREAD_TOTAL_SZIE)

#define LOCALGROUP_RATIO (SUBSURFACE_GROUP_SIZE/THREAD_SIZE_1D)

#if !TEXTURE_CACHE_DISABLED 
groupshared float4 SharedSubsurfaceDiffuseLighting[LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE];

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
groupshared float3 SharedSubsurfaceWorldNormal[LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE];
#endif

groupshared uint   SharedSubsurfaceProfileID[LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE];
#endif

// Ref: https://www.shadertoy.com/view/4sscDn
static int Masks[] =
{
	0x55555555, 0x33333333, 0x0F0F0F0F, 0x00FF00FF, 0x0000FFFF
};

// Ref: https://github.com/Forceflow/libmorton/blob/master/libmorton/include/morton2D_LUTs.h
static uint MortonLUTX[] =
{
	0,1,0,1,2,3,2,3,0,1,0,1,2,3,2,3,
	4,5,4,5,6,7,6,7,4,5,4,5,6,7,6,7,
	0,1,0,1,2,3,2,3,0,1,0,1,2,3,2,3,
	4,5,4,5,6,7,6,7,4,5,4,5,6,7,6,7
};

static uint MortonLUTY[] =
{
	0,0,1,1,0,0,1,1,2,2,3,3,2,2,3,3,
	0,0,1,1,0,0,1,1,2,2,3,3,2,2,3,3,
	4,4,5,5,4,4,5,5,6,6,7,7,6,6,7,7,
	4,4,5,5,4,4,5,5,6,6,7,7,6,6,7,7
};

// We can move this to cache instead of computing
uint2 ConvertGroupIndexTo2DUsingMoltonCode(uint GroupIndex, uint2 StartOffset)
{
#if MORTON_USE_LUT
	return uint2(MortonLUTX[GroupIndex], MortonLUTY[GroupIndex]) + StartOffset;
#else
	int n = 1;
	uint2 I = uint2(GroupIndex, GroupIndex >> 1) & Masks[0];
	UNROLL for (int i = 1; i <= 4; ++i)
	{
		I = (I | (I >> n)) & Masks[i];
		n *= 2;
	}
	return I + StartOffset;
#endif
}

uint2 ConvertGroupIndexToNormal2DGrid(uint LocalGroupIndex, uint2 StartOffset)
{
	return uint2(LocalGroupIndex % THREAD_SIZE_1D,
		LocalGroupIndex / THREAD_SIZE_1D) + StartOffset;
}

// BufferUV
float2 ConvertGridPos2UV(uint2 GridPosition)
{
	float2 GripPositionF = float2(GridPosition);
	return Output_ExtentInverse * (GripPositionF + 0.5f);
}
// Convert UV to groupThreadIds
float2 ConvertUVOffset2GTIDOffsetForInput0(float2 UVOffset)
{
	return UVOffset * Output_Extent;
}

bool IsIDInsideLocalShared(float2 GroupThreadId)
{
	return GroupThreadId.x >= -THREAD_TEXTURE_BORDER &&
		(GroupThreadId.x < (LARGE_GROUP_TEXTURE_DIMENSION_SIZE - THREAD_TEXTURE_BORDER)) &&
		GroupThreadId.y >= -THREAD_TEXTURE_BORDER &&
		(GroupThreadId.y < (LARGE_GROUP_TEXTURE_DIMENSION_SIZE - THREAD_TEXTURE_BORDER));
}

#if !TEXTURE_CACHE_DISABLED
BurleySampleDiffuseNormal SampleSharedDiffuseNormal(float2 SampleGroupThreadId)
{
	BurleySampleDiffuseNormal Sample;
	Sample.DiffuseLighting = 0.0;
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	Sample.WorldNormal = 0.0;
#endif
	Sample.ProfileID = 0;

#if SUBSURFACE_SAMPLER_TYPE == 0
	// We use point sampling by now
	int2 Id = SampleGroupThreadId + THREAD_TEXTURE_BORDER;
	Sample.DiffuseLighting = SharedSubsurfaceDiffuseLighting[Id.x + Id.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	Sample.WorldNormal = SharedSubsurfaceWorldNormal[Id.x + Id.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
#endif
	Sample.ProfileID = SharedSubsurfaceProfileID[Id.x + Id.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
#elif SUBSURFACE_SAMPLER_TYPE == 1
	// SUBSURFACE_SAMPLER_TYPE is always 0, but keeping this code around for reference

	//ref: https://en.wikipedia.org/wiki/Bilinear_interpolation
	int2 Id00 = floor(SampleGroupThreadId) + THREAD_TEXTURE_BORDER;
	int2 Id11 = ceil(SampleGroupThreadId) + THREAD_TEXTURE_BORDER;
	int2 Id01 = int2(Id00.x, Id11.y);
	int2 Id10 = int2(Id11.x, Id00.y);
	float x = SampleGroupThreadId.x + THREAD_TEXTURE_BORDER - Id00.x;
	float y = SampleGroupThreadId.y + THREAD_TEXTURE_BORDER - Id11.y;
	{
		float4 Q00 = SharedSubsurfaceDiffuseLighting[Id00.x + Id00.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float4 Q01 = SharedSubsurfaceDiffuseLighting[Id01.x + Id01.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float4 Q10 = SharedSubsurfaceDiffuseLighting[Id10.x + Id10.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float4 Q11 = SharedSubsurfaceDiffuseLighting[Id11.x + Id11.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		Sample.DiffuseLighting = Q00 * (1 - x)*(1 - y) + Q10 * x*(1 - y) + Q01 * (1 - x)*y + Q11 * x*y;
	}

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	{
		float3 Q00 = SharedSubsurfaceWorldNormal[Id00.x + Id00.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float3 Q01 = SharedSubsurfaceWorldNormal[Id01.x + Id01.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float3 Q10 = SharedSubsurfaceWorldNormal[Id10.x + Id10.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		float3 Q11 = SharedSubsurfaceWorldNormal[Id11.x + Id11.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
		Sample.WorldNormal = normalize(Q00 * (1 - x)*(1 - y) + Q10 * x*(1 - y) + Q01 * (1 - x)*y + Q11 * x*y);
	}
#endif

	Sample.ProfileID = SharedSubsurfaceProfileID[Id00.x + Id00.y*LARGE_GROUP_TEXTURE_DIMENSION_SIZE];
#endif

	return Sample;
}
#endif

#endif

// If we support independent samplers, use a point sampler as described below. But if they are not supported,
// then default to the regular sampler which will cause artifacts, but is better than not compiling.
// The bilinear sampler is required to reduce the variance overestimation with control variates. The point 
// sampler works but with a little worse performance.
#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SharedBurleyPointSampler SubsurfaceSampler1
	#define SharedBurleyBilinearSampler SubsurfaceSampler3
#else
	#define SharedBurleyPointSampler SubsurfaceSampler0
	#define SharedBurleyBilinearSampler SharedBurleyPointSampler
#endif

float3 GetBilateralNormal(float2 ClampedUV, float2 Extent)
{
	//BRANCH to avoid extra evaluation of the GBuffer.
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	#if SUBSTRATE_GBUFFER_FORMAT==1
	const uint2 BufferPos = ClampedUV * Extent;
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(BufferPos, 0)));
	return TopLayerData.WorldNormal;
	#else
	return GetScreenSpaceData(ClampedUV).GBuffer.WorldNormal;
	#endif
#elif BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH
	return BILATERAL_FILTER_DEFAULT_NORMAL;
#endif
}

float2 ClampUVLevel(float2 UV, float2 MinUV, float2 MaxUV, float MipLevel)
{
	uint CeilMipLevel = (uint)(ceil(MipLevel));
	float2 MipUVCorrection = float((1u << (1u + CeilMipLevel)) - 2u) * SubsurfaceInput0_ExtentInverse;
	
	float2 MinMipUVCorrection = 
		float2(SubsurfaceInput0_UVViewportMin.x == 0.0f ? 0.0f : MipUVCorrection.x,
			   SubsurfaceInput0_UVViewportMin.y == 0.0f ? 0.0f : MipUVCorrection.y);
	float2 MaxMipUVCorrection =
		float2(SubsurfaceInput0_UVViewportMax.x == 1.0f ? 0.0f : MipUVCorrection.x,
			   SubsurfaceInput0_UVViewportMax.y == 1.0f ? 0.0f : MipUVCorrection.y);

	return clamp(UV, MinUV + MinMipUVCorrection, MaxUV - MaxMipUVCorrection);
}

BurleySampleDiffuseNormal SampleSSSColorConsideringLocalShared(float2 CenterUV, float2 UVOffset, uint2 CenterGroupThreadID, float MipLevel)
{
	// Set mip level to 0 if the mipmap is not generated
	if (!ShouldGenerateMipmaps(SUBSURFACE_TILE_TYPE_AFIS))
	{
		MipLevel = 0.0f;
	}

	// Fix border flickering when mipmaps got garbage data.
	float2 ClampedUV = ClampUVLevel(CenterUV + UVOffset, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax, MipLevel);

#if !(ENABLE_PROFILE_ID_CACHE)
	#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceHeader SSSHeader = LoadSubstrateSSSHeader(ClampedUV);
	#else
	const FGBufferData SSSHeader = GetScreenSpaceData(ClampedUV).GBuffer;
	#endif
#endif

	// Burley works only with point sampler when we support world unit scale. Bilinear and trilinear will create artifacts.
	// So we use SubsurfaceSampler1 here instead of using SubsurfaceSampler0, which is a point sampler.
	// The reason that we can have different sampler for SubsurfaceSampler0 is that we have Separable running in the same pass.

	BurleySampleDiffuseNormal Sample;
	Sample.DiffuseLighting = 0.0;
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	Sample.WorldNormal = 0.0;
#endif

#if SUBSURFACE_BURLEY_COMPUTE

#if TEXTURE_CACHE_DISABLED
	Sample.DiffuseLighting = Texture2DSampleLevel(SubsurfaceInput0_Texture, SharedBurleyPointSampler, ClampedUV, MipLevel);

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	Sample.WorldNormal.xyz = GetBilateralNormal(ClampedUV, SubsurfaceInput0_Extent);
#endif

#if ENABLE_PROFILE_ID_CACHE
	Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(ClampedUV,Sample.DiffuseLighting.w);
#else
	Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(SSSHeader);
#endif

	return Sample;
#else
	float2 SampleGroupThreadId = ConvertUVOffset2GTIDOffsetForInput0(UVOffset) + CenterGroupThreadID; // Subtract 0.5 is to make sampling match. (0,0), samples at 1/w*0.5;
	bool bUseLocalShared = MipLevel == 0 && IsIDInsideLocalShared(SampleGroupThreadId);// We will have artifacts if we do not limit the miplevel

	BRANCH
		if (bUseLocalShared)
		{
			Sample = SampleSharedDiffuseNormal(SampleGroupThreadId);
			return Sample;
		}
		else
		{
			Sample.DiffuseLighting = Texture2DSampleLevel(SubsurfaceInput0_Texture, SharedBurleyPointSampler, ClampedUV, MipLevel);
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
			Sample.WorldNormal.xyz = GetBilateralNormal(ClampedUV, SubsurfaceInput0_Extent);
#endif

#if ENABLE_PROFILE_ID_CACHE
			Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(ClampedUV, Sample.DiffuseLighting.w);
#else
			Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(SSSHeader);
#endif

			return Sample;
		}
#endif

#else
	Sample.DiffuseLighting = Texture2DSampleLevel(SubsurfaceInput0_Texture, SharedBurleyPointSampler, ClampedUV, MipLevel);

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
	Sample.WorldNormal.xyz = GetBilateralNormal(ClampedUV, SubsurfaceInput0_Extent);
#endif
	
#if ENABLE_PROFILE_ID_CACHE
	Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(ClampedUV, Sample.DiffuseLighting.w);
#else
	Sample.ProfileID = ExtractSubsurfaceProfileIntWithInvalid(SSSHeader);
#endif

	return Sample;
#endif
}

float2 CalculateBurleyScale(float WorldUnitScale, float DepthAtCenter)
{
	float2 BurleyScale = WorldUnitScale;

	float SSSScaleX = SubsurfaceParams.x;
	BurleyScale *= SSSScaleX / DepthAtCenter;
	
	// cast from cm to mm for depth, and remove the effect of SUBSURFACE_KERNEL_SIZE. 
	BurleyScale *= SUBSURFACE_KERNEL_SIZE / BURLEY_CM_2_MM;

	// account for Screen Percentage/Dyanmic Resolution Scaling
	BurleyScale *= (SubsurfaceInput0_ViewportSize.x * SubsurfaceInput0_ExtentInverse.x);
	BurleyScale.y *= (SubsurfaceInput0_Extent.x * SubsurfaceInput0_ExtentInverse.y);

	return BurleyScale;
}

// Given the Depth and the BurleyParameter, figure out the actual radius of the center pixel in MM,
// taking into account the depth and screen dimensions.
float CalculateCenterSampleRadiusInMM(FBurleyParameter BurleyParameter, float Depth)
{
	float DiffuseMeanFreePath = GetDiffuseMeanFreePathForSampling(BurleyParameter.DiffuseMeanFreePath);

	float A = GetComponentForScalingFactorEstimation(BurleyParameter.SurfaceAlbedo);
	float S = GetScalingFactor(A);
	float3 S3D = GetScalingFactor3D(BurleyParameter.SurfaceAlbedo.xyz);

	float2 BurleyScale = CalculateBurleyScale(BurleyParameter.WorldUnitScale,Depth);

	// In the reference function, UVOffset = BurleyScale * RadiusInMM
	//      float2 UVOffset = BurleyScale*BurleySampleInfo.RadiusInMM;
	// So, given the UV offset, we can find the distance in mm as:
	//      float DistInMM = UvOffset.x/BurleyScale.x + UvOffset.y/BurleyScale.y;
	// But for stability, we can just average them.
	float CenterSampleRadiusInMM = 0.5f * (SubsurfaceInput0_ExtentInverse.x/BurleyScale.x + SubsurfaceInput0_ExtentInverse.y/BurleyScale.y);

	return CenterSampleRadiusInMM;
}

// Given the UV and BurleyParameter, determine how much RGB weight should be assigned to the center 
// pixel. The rest of the weight would be applied from the blur.
float3 CalculateCenterSampleWeight(float Depth, FBurleyParameter BurleyParameter)
{
	float CenterSampleRadiusInMM = CalculateCenterSampleRadiusInMM(BurleyParameter, Depth);

	float DiffuseMeanFreePath = GetDiffuseMeanFreePathForSampling(BurleyParameter.DiffuseMeanFreePath);

	// To calculate the surface free path from albedo, use the default scaling.
	float3 D = DiffuseMeanFreePath / GetScalingFactor3D(BurleyParameter.SurfaceAlbedo.xyz);

	float3 CenterSampleWeight;

	CenterSampleWeight.x = GetCDF(D.x,CenterSampleRadiusInMM,0);
	CenterSampleWeight.y = GetCDF(D.y,CenterSampleRadiusInMM,0);
	CenterSampleWeight.z = GetCDF(D.z,CenterSampleRadiusInMM,0);

	return CenterSampleWeight;
}

void UpdateSeed(int3 Seed3D, inout int StartSeed)
{
	/*To make R2Sequence work, we need to rebase the R2 sequence start index to a new one uniformly in the R2 space,
	  then sample sequentially for the current frame. With this mechanism, we can get the best
	  quality for each frame, and thus best over time.*/
#if SAMPLE_ROOT_ANGLE_R2SEQUENCE
	StartSeed = Rand3DPCG16(int3(Seed3D.xy, StartSeed)).x;
#endif
}

float4 BurleyNormalizedSS(float2 BufferUV, uint2 GroupThreadID)
{
	BurleySampleDiffuseNormal CenterSample = SampleSSSColorConsideringLocalShared(BufferUV, 0, GroupThreadID, 0);
	float DepthAtDiscCenter = CenterSample.DiffuseLighting.w;

	float3 OriginalColor = CenterSample.DiffuseLighting.rgb;

	float4 OutColor = 0;

	BRANCH if (DepthAtDiscCenter <= 0)
	{
		return OutColor;
	}

#if SUBSTRATE_GBUFFER_FORMAT==1
	const FSubstrateSubsurfaceData SSSData = LoadSubstrateSSSData(BufferUV);
	const FSubstrateTopLayerData TopLayerData = LoadSubstrateTopLayerData(BufferUV);
	const float3 WorldNormal = TopLayerData.WorldNormal;
	const uint SubsurfaceProfileInt = SubstrateSubSurfaceHeaderGetProfileId(SSSData.Header);
	const FBurleyParameter BurleyParameter = GetBurleyParameters(SSSData);
#else
	const FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(BufferUV);
	const float3 WorldNormal = ScreenSpaceData.GBuffer.WorldNormal;
	const uint SubsurfaceProfileInt = ExtractSubsurfaceProfileInt(ScreenSpaceData.GBuffer);
	const FBurleyParameter BurleyParameter = GetBurleyParameters(SubsurfaceProfileInt, ScreenSpaceData.GBuffer);
#endif

	float DiffuseMeanFreePathForSampling = GetDiffuseMeanFreePathForSampling(BurleyParameter.DiffuseMeanFreePath);
	float A = GetComponentForScalingFactorEstimation(BurleyParameter.SurfaceAlbedo);
	float3 BoundaryColorBleed = GetSubsurfaceProfileBoundaryColorBleed(SubsurfaceProfileInt).xyz;

	float S = GetScalingFactor(A);
	float3 S3D = GetScalingFactor3D(BurleyParameter.SurfaceAlbedo.xyz);

	int SeedStart = View.FrameNumber;
	float3 WeightingFactor = 0.0f;
	float4 RadianceAccumulated = float4(0.0f, 0.0f, 0.0f, 1.0f);
	float Mask = 1.0f;
	float3 BoundaryColorBleedAccum = float3(0.0f, 0.0f, 0.0f);

#if SUBSURFACE_SINGLE_PASS
	int NumOfSamples = BURLEY_NUM_SAMPLES;
	float InvNumOfSamples = BURLEY_INV_NUM_SAMPLES;
#else
	int NumOfSamples = GetNumOfSamplesBasedOnQuality(BufferUV);
	float InvNumOfSamples = 1.0f / NumOfSamples;
#endif

	const int SSSOverrideNumSamples = SubsurfaceParams.z;
	if (SSSOverrideNumSamples > 0)
	{
		NumOfSamples = SSSOverrideNumSamples;
		InvNumOfSamples = 1.0f / float(SSSOverrideNumSamples);
	}

	int3  Seed3D = int3(BufferUV*SubsurfaceInput0_Extent, 0);
	UpdateSeed(Seed3D,SeedStart);

#if !USE_BILATERAL_FILTERING
	float ActiveNumOfSamples = 0;
#endif

	float2 BurleyScale = CalculateBurleyScale(BurleyParameter.WorldUnitScale,DepthAtDiscCenter);

	/*************************************************************************************
	 * Center Sample Reweighting
	 * 
	 * The original burley algorithm involes monte car sampling. Given a random variable [0,1],
	 * find the distance of that point from the center using the CDF, and then divide by PDF. 
	 * But it is somewhat inefficient because it is weighted heavily towards the center.
	 *
	 * Instead, we are going to split the [0,1] random variable range. First, we figure out the
	 * radius (R) of the center sample in world space. Second, we are going to determine the random
	 * variable (T) such that CDF(R) = T. Then we split the range into two segments.
	 *
	 * 1. The center sample, which include the random variable values from [0,T].
	 * 2. All other samples, which include the random variable values from [T,1].
	 *
	 * With the center sample is scaled the weight T and the rest of the samples are weighted
	 * by (1-T). There shouldn't be any bias, except for small errors due to precision.
	 **************************************************************************************/

#if REWEIGHT_CENTER_SAMPLE
	float CenterSampleRadiusInMM = CalculateCenterSampleRadiusInMM(BurleyParameter, BurleyScale, SubsurfaceInput0_ExtentInverse);
	float CenterSampleRadiusCdf = CalculateCenterSampleCdf(BurleyParameter, CenterSampleRadiusInMM);
	float3 CenterSampleWeight = CalculateCenterSampleWeight(DepthAtDiscCenter, BurleyParameter);
#endif

	LOOP for (int i = 0; i < NumOfSamples; ++i)
	{
		// Step 1: sample generation
		// Create an 2d disk sampling pattern (we can load from the disk as a texture or buffer).
		Seed3D.z = SeedStart++;
		float2 Random0T1 = Generate2DRandomNumber(Seed3D);

#if REWEIGHT_CENTER_SAMPLE
		// The random variable goes from 0 to 1. CenterSampleRadiusCdf is the probability that a sample hits the
		// center pixel. Since that probability is accounted for in the lighting, we only sample in the
		// range [CenterSampleRadiusCdf,1] instead of [0,1]
		Random0T1.x = CenterSampleRadiusCdf + Random0T1.x*(1.0f - CenterSampleRadiusCdf);
#endif

		FBurleySampleInfo BurleySampleInfo = GenerateSampleInfo(Random0T1, DiffuseMeanFreePathForSampling, S, i);

		// Step 2: get the light radiance and depth at the offset
		// and estimate the scale from the random disk sampling space to sceen space.
		
		// World unit to screen space unit
		float2 UVOffset = BurleyScale*BurleySampleInfo.RadiusInMM;
		UVOffset.x *= BurleySampleInfo.CosTheta;
		UVOffset.y *= BurleySampleInfo.SinTheta;

		// Sampling
		{
			float2 SampledDiscUV = BufferUV + UVOffset;

#if SUBSURFACE_SINGLE_PASS
			SDiffuseAndSpecular SampledDiffuseAndSpecular = ReconstructLighting(SampledDiscUV, ReconstructMethod);
			float4 SampledRadianceAndDepth = float4(SampledDiffuseAndSpecular.Diffuse, CalcSceneDepth(SampledDiscUV));

		#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateTopLayerData TopLayerData = LoadSubstrateTopLayerData(SampleDiscUV);
			const float3 SampleWorldNormal = TopLayerData.WorldNormal;
		#else
			const FScreenSpaceData SampleScreenSpaceData = GetScreenSpaceData(SampledDiscUV);
			const float3 SampleWorldNormal = SampleScreenSpaceData.GBuffer.WorldNormal;
		#endif

			uint LocalProfile = ExtractSubsurfaceProfileIntWithInvalid(GetSubsurfaceProfileId(SampledDiscUV));
#else

			// Determine the miplevel with the expected number of samples at the pixel.
			// how much does one pixel cover in real world at a distance.
			float texSize = BurleyScale.x * BurleyScale.y;
			float MipLevel = 0.5*max(-log2(MIP_CONSTANT_FACTOR*NumOfSamples*BurleySampleInfo.Pdf/(DiffuseMeanFreePathForSampling*DiffuseMeanFreePathForSampling*texSize)), 0);

			// Code used to output miplevels
#if DEBUG_MIP_LEVEL
			OutColor.xyz = float3(DiffuseMeanFreePathForSampling, texSize, BurleySampleInfo.Pdf);
			OutColor.w = 2 + (MipLevel);
			return OutColor;
#endif
			// If we are using half resolution,we should shift the mip level by -1
#if SUBSURFACE_HALF_RES
			MipLevel -= 1;
#endif
			// We cannot use trilinear for irradiance mipmaps, it brings artifacts when artist changes the dmfp
			// So we use ceil to use the mips of the next level.
			MipLevel = clamp(ceil(MipLevel), 0, 5);

			BurleySampleDiffuseNormal FoundSample = SampleSSSColorConsideringLocalShared(BufferUV, UVOffset, GroupThreadID, MipLevel);
			float4 SampledRadianceAndDepth = FoundSample.DiffuseLighting;
#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
			float3 SampleWorldNormal = FoundSample.WorldNormal;
#else
			float3 SampleWorldNormal = float3(1.0f,0.0f,0.0f);
#endif
			uint LocalProfile = FoundSample.ProfileID;
#endif

			// Step 3: Get weight from normal similarity
			float NormalWeight = sqrt(saturate(dot(SampleWorldNormal,WorldNormal)*.5f + .5f));

			// Step 4: create the bilateral filtering weighted Distance between entry and exit.
#if USE_BILATERAL_FILTERING
			// Bring DeltaDepth into the normalized kernal space.
			// 
			// Without the division of world unit scale, we add too much penalty to the sample weight when world unit scale is 
			// large. E.g., when we have a 1 cm world unit scale (i.e., 1cm is regarded as 1mm), if we get 1mm depth difference, 
			// it should be treated as 0.1mm instead of 1mm to reduce the weight contribution.
			float DeltaDepth = (SampledRadianceAndDepth.w - DepthAtDiscCenter) * BURLEY_CM_2_MM / BurleyParameter.WorldUnitScale;
			float RadiusSampledInMM = sqrt(BurleySampleInfo.RadiusInMM * BurleySampleInfo.RadiusInMM + DeltaDepth * DeltaDepth);

#if RESAMPLE_PDF
			BurleySampleInfo.Pdf = GetPdf(RadiusSampledInMM, DiffuseMeanFreePathForSample, S);
#endif
#else
			float RadiusSampledInMM = BurleySampleInfo.RadiusInMM;
#endif				

			// Determine the tint color, if the sampling pixel is not subsurface, we use tint color
			// to mask out the sampling. Unless we specifically want the shadowing region.
			BoundaryColorBleedAccum += (LocalProfile == SubsurfaceProfileInt || LocalProfile == SSS_PROFILE_ID_INVALID) ? 1.0f : BoundaryColorBleed;
			Mask = (LocalProfile != SSS_PROFILE_ID_INVALID) ? 1 : 0;

			// Step 4: accumulate radiance from the diffusion profile rR(r)
			// make sure the DiffuseMeanFreePath is not zero and in mm.
			float3 DiffusionProfile = GetDiffuseReflectProfileWithDiffuseMeanFreePath(BurleyParameter.DiffuseMeanFreePath.xyz, S3D.xyz, RadiusSampledInMM);
			float3 SampleWeight = (DiffusionProfile / BurleySampleInfo.Pdf) * Mask * NormalWeight;

			RadianceAccumulated.xyz += SampleWeight * (SampledRadianceAndDepth.xyz);

#if USE_BILATERAL_FILTERING
			WeightingFactor += SampleWeight;
#else
			ActiveNumOfSamples += Mask;
#endif	
		}
	}

	// 0.99995f is a compensitation to make it energe conservation.
	const float EnergyNormalization = 1.0f / 0.99995f;


#if (RADIUS_SAMPLE_UNIFORM_DISK)
	RadianceAccumulated.xyz *= (InvSampleCount*0.5 * 2 * PI);
#elif !USE_BILATERAL_FILTERING
	RadianceAccumulated.xyz *= (ActiveNumOfSamples==0)? 0 :(1/ActiveNumOfSamples * 2 * PI) * EnergyNormalization;
#else
	// The added epsilon is used to avoid divid by zero.
	RadianceAccumulated.xyz *= select(WeightingFactor == 0, 0.0, 1.0f /WeightingFactor*EnergyNormalization);
#endif

	RadianceAccumulated.xyz *= BoundaryColorBleedAccum*InvNumOfSamples;

#if REWEIGHT_CENTER_SAMPLE
	// Apply lerp with center pixel
	RadianceAccumulated.xyz = lerp(RadianceAccumulated.xyz,OriginalColor,CenterSampleWeight);
#endif

	// The opacity works by reducing the radius based on opacity, but this runs into precision issues with low opacity values.
	// So as the opacity goes to SSSS_OPACITY_THRESHOLD_EPS, we transition to fully disabling SSS by the time we get there.
	float LowOpacityEps = SSSS_OPACITY_THRESHOLD_EPS;

	float OriginalLerp = saturate((BurleyParameter.SurfaceOpacity - LowOpacityEps) / LowOpacityEps);

	OutColor.xyz = lerp(OriginalColor,RadianceAccumulated.xyz,OriginalLerp);
	OutColor.w = NumOfSamples;

	return OutColor;
}

float4 UpdateQualityVariance(float4 SubsurfaceColor, float2 BufferUV)
{
	float WeightFinal = EXPONENTIAL_WEIGHT;
	FHistoryState HistoryState = GetHistoryState(BufferUV);

	float4 History = HistoryState.History;
	if (HistoryState.OffScreen)
	{
		WeightFinal = 1.0f;
	}

	float4 NewHistory = (float4)0;

	// Use the bilinear jitter-corrected lighting as the control variable (Constant CV coefficient = 1.0) to remove the variance over-estimation
	// of spatial features, like peach fuzz, inside subsurface scattering region. It will not affect the output surface color.
	float4 BilinearSurfaceColor = Texture2DSample(SubsurfaceInput0_Texture, SharedBurleyBilinearSampler, BufferUV);
	float2 BufferUVJitter = (ScreenPosToViewportUV(View.TemporalAAJitter.xy)-float2(0.5, 0.5))* SubsurfaceInput0_UVViewportSize.xy;
	float2 BufferUVJitterRemoved = clamp(BufferUV - BufferUVJitter, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax);
	float4 ControlVariable = Texture2DSampleLevel(SubsurfaceInput3_Texture, SharedBurleyBilinearSampler, BufferUVJitterRemoved, 0);

	// Calculate the illuminance
	float NewSurfaceLuminanceResidual = Luminance(pow(BilinearSurfaceColor.rgb, 1 / 2.2))- Luminance(pow(ControlVariable.rgb, 1 / 2.2));

	NewHistory.a = (1 - WeightFinal) * History.a + WeightFinal * NewSurfaceLuminanceResidual;

	float Delta = NewSurfaceLuminanceResidual - History.a;
	NewHistory.b = (1 - WeightFinal)*History.b + WeightFinal * (1 - WeightFinal)*Delta*Delta;

#if USE_TRUE_DISTRIBUTION_VAR
	NewHistory.g = (1- WeightFinal)*History.g+WeightFinal * SubsurfaceColor.a;
#else
	// Update the random variable variance
	NewHistory.g = History.b*SubsurfaceColor.a;
#endif
	// Update the average number of samples used at each pixel (only for visualization purpose)
	NewHistory.r = (1 - WeightFinal) * History.r + WeightFinal * SubsurfaceColor.a / BURLEY_NUM_SAMPLES;


	BRANCH if (SubsurfaceColor.a <= 0)
	{
		NewHistory = 0;
	}

	return NewHistory;
}

#if SUBSURFACE_BURLEY_COMPUTE

// Compute shader common data and functions

RWTexture2D<float4> SSSColorUAV;
RWTexture2D<float4> HistoryUAV;

void BurleyComputeMain(uint2 DT_ID, uint2 G_ID, uint GI)
{

	int2 TopLeftCorner = G_ID * SUBSURFACE_GROUP_SIZE - THREAD_TEXTURE_BORDER + Output_ViewportMin;

	// Step 1: Read the diffuse lighting into the local share. 64+64+16, three cycles
#if SUBSURFACE_PASS == SUBSURFACE_PASS_ONE

#if !TEXTURE_CACHE_DISABLED

	// We do not need to use unroll if we have a fixed number of thread dimension.
	{
		UNROLL for (uint i = 0; i < NUM_OF_PASS_REQUIRED_FILL_SHARED_DIFFUSE_TEXTURE; i++)
		{
			// Calculate the sample uv for the current thread
			uint LocalSharedIndex = GI * NUM_OF_PASS_REQUIRED_FILL_SHARED_DIFFUSE_TEXTURE + i;
			BRANCH if (LocalSharedIndex >= LARGE_GROUP_TEXTURE_SHARE_TOTALSIZE)
			{
				break;
			}

			float2 SampleUV = Output_ExtentInverse * (TopLeftCorner + float2(LocalSharedIndex % LARGE_GROUP_TEXTURE_DIMENSION_SIZE,
				LocalSharedIndex / LARGE_GROUP_TEXTURE_DIMENSION_SIZE) + 0.5f);
			SampleUV = clamp(SampleUV, SubsurfaceInput0_UVViewportBilinearMin, SubsurfaceInput0_UVViewportBilinearMax);

			float4 SubsurfaceColorWithSSSIndicator = Texture2DSample(SubsurfaceInput0_Texture, SubsurfaceSampler0, SampleUV);
			SharedSubsurfaceDiffuseLighting[LocalSharedIndex] = SubsurfaceColorWithSSSIndicator;

			// Also fetch the normal and profile ID
		#if SUBSTRATE_GBUFFER_FORMAT==1
			const FSubstrateTopLayerData TopLayerData = LoadSubstrateTopLayerData(SampleUV);
			const float3 SampleWorldNormal = TopLayerData.WorldNormal;
			const FSubstrateSubsurfaceHeader SampleSSSHeader = LoadSubstrateSSSHeader(SampleUV);
		#else
			const FGBufferData SampleSSSHeader = GetScreenSpaceData(SampleUV).GBuffer;
			const float3 SampleWorldNormal = SampleSSSHeader.WorldNormal;
		#endif

#if BILATERAL_FILTER_KERNEL_FUNCTION_TYPE == BILATERAL_FILTER_KERNEL_FUNCTION_DEPTH_AND_NORMAL
			SharedSubsurfaceWorldNormal[LocalSharedIndex] = SampleWorldNormal;
#endif
			SharedSubsurfaceProfileID[LocalSharedIndex] = ExtractSubsurfaceProfileIntWithInvalid(SampleSSSHeader);
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// need to perform burley normalized subsurface scattering if it's burley
	// this would unroll to just 1 sequence.
	{
		UNROLL for (uint i = 0; i < LOCALGROUP_RATIO; ++i)
		{
			UNROLL for (uint j = 0; j < LOCALGROUP_RATIO; ++j)
			{
				uint2 Pos = ConvertGroupIndexToNormal2DGrid(GI, TopLeftCorner + THREAD_TEXTURE_BORDER + uint2(j, i)*THREAD_SIZE_1D);
				float2 LBufferUV = ConvertGridPos2UV(Pos);
				SSSColorUAV[Pos] = BurleyNormalizedSS(LBufferUV, Pos - (TopLeftCorner + THREAD_TEXTURE_BORDER));
			}
		}
	}

#else
	UNROLL for (uint i = 0; i < LOCALGROUP_RATIO; ++i)
	{
		UNROLL for (uint j = 0; j < LOCALGROUP_RATIO; ++j)
		{

			uint2 Pos = ConvertGroupIndexToNormal2DGrid(GI, TopLeftCorner + THREAD_TEXTURE_BORDER + uint2(j, i)*THREAD_SIZE_1D);
			float2 LBufferUV = ConvertGridPos2UV(Pos);
			SSSColorUAV[Pos] = BurleyNormalizedSS(ConvertGridPos2UV(Pos), Pos - (TopLeftCorner + THREAD_TEXTURE_BORDER));
		}
	}
#endif
#else

	uint2 Pos = DT_ID.xy*SUBSURFACE_GROUP_SIZE / THREAD_SIZE_1D + Output_ViewportMin;

	//we only update the variance if the subsurface is burley
	UNROLL for (uint i = 0; i < LOCALGROUP_RATIO; ++i)
	{
		UNROLL for (uint j = 0; j < LOCALGROUP_RATIO; ++j)
		{
			uint2 TargetGridPos = Pos + uint2(i, j);
			float2 LBufferUV = ConvertGridPos2UV(TargetGridPos);
			// We perform quality variance thread-wise in the second pass
			float4 SurfaceColor = Texture2DSample(SubsurfaceInput0_Texture, SubsurfaceSampler0, LBufferUV);

			// We have to check per pixel if this is a burley so that we don't overwrite separable pixels
			uint SelectedProfile = GetSubsurfaceProfileId(LBufferUV);
			bool UseBurley = GetSubsurfaceProfileUseBurley(SelectedProfile);
			if (UseBurley)
			{
				HistoryUAV[TargetGridPos] = UpdateQualityVariance(SurfaceColor, LBufferUV);
				SSSColorUAV[TargetGridPos] = float4(SurfaceColor.rgb, 1.0f);
			}
		}
	}
#endif
}

#endif
