// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	SubsurfaceProfileCommon.ush: Subsurface scattering parameter lookup / decoding constants.
=============================================================================================*/

#pragma once

///////////////////////////////////////////////////////////////////////////////////////////////
// Describe SSS profile texture (data offsets & size)
// NOTE: Changing offsets below requires updating all instances of #SSSS_CONSTANTS
// TODO: This needs to be defined in a single place and shared between C++ and shaders!
#define SSSS_TINT_SCALE_OFFSET					0
#define BSSS_SURFACEALBEDO_OFFSET               (SSSS_TINT_SCALE_OFFSET+1)
#define BSSS_DMFP_OFFSET                        (BSSS_SURFACEALBEDO_OFFSET+1)
#define SSSS_TRANSMISSION_OFFSET				(BSSS_DMFP_OFFSET+1)
#define SSSS_BOUNDARY_COLOR_BLEED_OFFSET		(SSSS_TRANSMISSION_OFFSET+1)
#define SSSS_DUAL_SPECULAR_OFFSET				(SSSS_BOUNDARY_COLOR_BLEED_OFFSET+1)
#define SSSS_KERNEL0_OFFSET						(SSSS_DUAL_SPECULAR_OFFSET+1)
#define SSSS_KERNEL0_SIZE						13
#define SSSS_KERNEL1_OFFSET						(SSSS_KERNEL0_OFFSET + SSSS_KERNEL0_SIZE)
#define SSSS_KERNEL1_SIZE						9
#define SSSS_KERNEL2_OFFSET						(SSSS_KERNEL1_OFFSET + SSSS_KERNEL1_SIZE)
#define SSSS_KERNEL2_SIZE						6
#define SSSS_KERNEL_TOTAL_SIZE					(SSSS_KERNEL0_SIZE + SSSS_KERNEL1_SIZE + SSSS_KERNEL2_SIZE)
#define BSSS_TRANSMISSION_PROFILE_OFFSET		(SSSS_KERNEL0_OFFSET + SSSS_KERNEL_TOTAL_SIZE)
#define BSSS_TRANSMISSION_PROFILE_SIZE			32
#define	SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE	5.0f // See MaxTransmissionProfileDistance in ComputeTransmissionProfile(), SeparableSSS.cpp
#define SSSS_MAX_DUAL_SPECULAR_ROUGHNESS		2.0f

// Threshold value at which model switches from SSS to default lit
#define SSSS_OPACITY_THRESHOLD_EPS 0.10

///////////////////////////////////////////////////////////////////////////////////////////////
// Burley constants
#define BURLEY_MM_2_CM		0.1f
#define BURLEY_CM_2_MM      10.0f

// exactly one of these should be true
#define LIGHT_PERPENDICULAR 0
#define LIGHT_DIFFUSESURFACE 0
#define LIGHT_PERPENDICULAR_DIFFUSE_SURFACE 1

// Consistent in BurleyNormalizedSSSCommon.ush and SubsurfaceProfile.cpp
#define SSS_TYPE_BURLEY	    0
#define SSS_TYPE_SSSS		1

// Make sure UIMax|ClampMax of WorldUNitScale * ENC_WORLDUNITSCALE_IN_CM_TO_UNIT <= 1
#define ENC_WORLDUNITSCALE_IN_CM_TO_UNIT 0.02f
#define DEC_UNIT_TO_WORLDUNITSCALE_IN_CM 1/ENC_WORLDUNITSCALE_IN_CM_TO_UNIT

// Make sure UIMax|ClampMax of DiffuseMeanFreePath * ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT <= 1
#define ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT (0.01f*0.2f)
#define DEC_UNIT_TO_DIFFUSEMEANFREEPATH_IN_MM 1/ENC_DIFFUSEMEANFREEPATH_IN_MM_TO_UNIT

// Make sure UIMax|ClampMax of ExtinctionScale * ENC_EXTINCTIONSCALE_FACTOR <= 1
#define ENC_EXTINCTIONSCALE_FACTOR	0.01f
#define DEC_EXTINCTIONSCALE_FACTOR  1/ENC_EXTINCTIONSCALE_FACTOR
///////////////////////////////////////////////////////////////////////////////////////////////
// Accessors

half4 GetSubsurfaceProfileTexture(uint SampleIndex, uint SubsurfaceProfileInt)
{
	// One profile per line, encoded using constants above. See FSubsurfaceProfileTexture::CreateTexture() in SubsurfaceProfile.cpp.
#if COMPILER_GLSL_ES3_1 // Force to use a point sampler for Texture2D.Load on OpenGLES platform
	return View.SSProfilesTexture.SampleLevel(View.SSProfilesSampler, (uint2(SampleIndex, SubsurfaceProfileInt) + float2(0.5f, 0.5f)) * View.SSProfilesTextureSizeAndInvSize.zw, 0);
#else
	return View.SSProfilesTexture.Load(int3(SampleIndex, SubsurfaceProfileInt, 0));
#endif 
}

half4 GetSubsurfaceProfileTexture(Texture2D InSSProfilesTexture, SamplerState InSSProfilesSampler, float4 InSSProfilesTextureSizeAndInvSize, uint SampleIndex, uint SubsurfaceProfileInt)
{
	// One profile per line, encoded using constants above. See FSubsurfaceProfileTexture::CreateTexture() in SubsurfaceProfile.cpp.
#if COMPILER_GLSL_ES3_1 // Force to use a point sampler for Texture2D.Load on OpenGLES platform
	return InSSProfilesTexture.SampleLevel(InSSProfilesSampler, (uint2(SampleIndex, SubsurfaceProfileInt) + float2(0.5f, 0.5f)) * InSSProfilesTextureSizeAndInvSize.zw, 0);
#else
	return InSSProfilesTexture.Load(int3(SampleIndex, SubsurfaceProfileInt, 0));
#endif 
}

void GetSubsurfaceProfileDualSpecular(uint SubsurfaceProfileInt, half Roughness, half Opacity, out half LobeRoughness0, out half LobeRoughness1, out half LobeMix)
{
	const half4 Data = GetSubsurfaceProfileTexture(SSSS_DUAL_SPECULAR_OFFSET, SubsurfaceProfileInt);

	// Smooth blend out dual specular when opacity is low, we have the extra SSSS_OPACITY_THRESHOLD_EPS so that we fade out by the time we
	// get to 0.01, as opposed to 0.0.
	half MaterialRoughnessToLobeRoughness0 = lerp(1.0f, Data.x * SSSS_MAX_DUAL_SPECULAR_ROUGHNESS, saturate((Opacity - SSSS_OPACITY_THRESHOLD_EPS) * 10.0f));
	half MaterialRoughnessToLobeRoughness1 = lerp(1.0f, Data.y * SSSS_MAX_DUAL_SPECULAR_ROUGHNESS, saturate((Opacity - SSSS_OPACITY_THRESHOLD_EPS) * 10.0f));
	LobeMix = Data.z;

	// Avoid specular explosion, and approximate `area lights`.
	LobeRoughness0 = max(saturate(Roughness * MaterialRoughnessToLobeRoughness0), 0.02f);
	LobeRoughness1 = saturate(Roughness * MaterialRoughnessToLobeRoughness1);
}

// Surface albedo and mean free path length
float4 GetSubsurfaceProfileSurfaceAlbedo(uint SubsurfaceProfileInt)
{
	return GetSubsurfaceProfileTexture(BSSS_SURFACEALBEDO_OFFSET, SubsurfaceProfileInt);
}

float4 GetSubsurfaceProfileDiffuseMeanFreePath(uint SubsurfaceProfileInt)
{
	return GetSubsurfaceProfileTexture(BSSS_DMFP_OFFSET, SubsurfaceProfileInt);
}

float GetSubsurfaceProfileWorldUnitScale(uint SubsurfaceProfileInt)
{
	return GetSubsurfaceProfileTexture(SSSS_TINT_SCALE_OFFSET, SubsurfaceProfileInt).a;
}

float DecodeWorldUnitScale(float EncodedWorldUnitScale)
{
	return EncodedWorldUnitScale * DEC_UNIT_TO_WORLDUNITSCALE_IN_CM;
}

float4 DecodeDiffuseMeanFreePath(float4 EncodedDiffuseMeanFreePath)
{
	return EncodedDiffuseMeanFreePath * DEC_UNIT_TO_DIFFUSEMEANFREEPATH_IN_MM;
}

float EncodeScatteringDistribution(float ScatteringDistribution)
{
	return (ScatteringDistribution + 1.0f) * 0.5f;
}

float DecodeScatteringDistribution(float ScatteringDistribution)
{
	return ScatteringDistribution * 2.0f - 1.0f;
}

float EncodeExtinctionScale(float ExtinctionScale)
{
	return ExtinctionScale * ENC_EXTINCTIONSCALE_FACTOR;
}

float DecodeExtinctionScale(float ExtinctionScale)
{
	return ExtinctionScale * DEC_EXTINCTIONSCALE_FACTOR;
}

bool GetSubsurfaceTransmittanceProfileUseBurley(uint SubsurfaceProfileInt)
{
	half Type = GetSubsurfaceProfileTexture(SSSS_BOUNDARY_COLOR_BLEED_OFFSET, SubsurfaceProfileInt).a;
	return abs(Type - SSS_TYPE_BURLEY) < 0.01f;
}

bool GetSubsurfaceProfileUseBurley(uint SubsurfaceProfileInt)
{
	half Type = GetSubsurfaceProfileTexture(SSSS_BOUNDARY_COLOR_BLEED_OFFSET, SubsurfaceProfileInt).a;
	return abs(Type - SSS_TYPE_BURLEY) < 0.01f;
}

bool GetSubsurfaceProfileUseSeparable(uint SubsurfaceProfileInt)
{
	half Type = GetSubsurfaceProfileTexture(SSSS_BOUNDARY_COLOR_BLEED_OFFSET, SubsurfaceProfileInt).a;
	return abs(Type - SSS_TYPE_SSSS) < 0.01f;
}

float4 GetSubsurfaceProfileDMFPInCm(int SubsurfaceProfileInt)
{
	const float4 DiffuseMeanFreePath = DecodeDiffuseMeanFreePath(GetSubsurfaceProfileDiffuseMeanFreePath(SubsurfaceProfileInt));
	const float WorldUnitScale = DecodeWorldUnitScale(GetSubsurfaceProfileWorldUnitScale(SubsurfaceProfileInt));
	return DiffuseMeanFreePath * WorldUnitScale; // In cm
}

#if SHADING_PATH_MOBILE
half CalculateCurvature(half3 WorldNormal, float3 WorldPosition)
{
#if 0
	half DeltaNormal = length(abs(DDX(WorldNormal)) + abs(DDY(WorldNormal)));
	half DeltaPosition = length(abs(DDX(WorldPosition)) + abs(DDY(WorldPosition))) * BURLEY_CM_2_MM;
	half CurvatureApprox = DeltaNormal / DeltaPosition;
#else
	half3 dNdx = ddx((HALF3_TYPE)WorldNormal);
	half3 dNdy = ddy((HALF3_TYPE)WorldNormal);
	half x = dot(dNdx, dNdx);
	half y = dot(dNdy, dNdy);
	half CurvatureApprox = pow(max(x, y), ResolvedView.NormalCurvatureToRoughnessScaleBias.z);
#endif
	return CurvatureApprox;
}
#endif