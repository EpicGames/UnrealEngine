// Copyright Epic Games, Inc. All Rights Reserved.

#define EYE_ADAPTATION_LOOSE_PARAMETERS 1

#include "Common.ush"
#include "Random.ush"
#include "EyeAdaptationCommon.ush"
#include "TextureSampling.ush"
#include "MonteCarlo.ush"
#include "Quantization.ush"

//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


//------------------------------------------------------- ENUM VALUES

/** Payload of the history. History might still have addtional TAA internals. */
	// Only have RGB.
	#define HISTORY_PAYLOAD_RGB 0
	
	// Have RGB and translucency in alpha.
	#define HISTORY_PAYLOAD_RGB_TRANSLUCENCY 1
	
	// Have RGB and opacity in alpha.
	#define HISTORY_PAYLOAD_RGB_OPACITY (HISTORY_PAYLOAD_RGB_TRANSLUCENCY)
	
	// Have RGB and DOF's CocRadius in alpha.
	#define HISTORY_PAYLOAD_RGB_COC 2
	
	// Have RGB, opacity in alpha and DOF's CocRadius in separate's Red.
	#define HISTORY_PAYLOAD_RGB_OPACITY_COC 3
	

/** Caching method for scene color. */
	// Disable any in code cache.
	#define AA_SAMPLE_CACHE_METHOD_DISABLE 0
	
	// Caches 3x3 Neighborhood into VGPR (although my have corner optimised away).
	#define AA_SAMPLE_CACHE_METHOD_VGPR_3X3 1
	
	// Prefetches scene color into 10x10 LDS tile (8x8 when screen percentage < 71%).
	#define AA_SAMPLE_CACHE_METHOD_LDS 2
	

/** Clamping method for scene color. */
	// Min max neighboorhing samples.
	#define HISTORY_CLAMPING_BOX_MIN_MAX 0
	
	// Variance computed from neighboorhing samples.
	#define HISTORY_CLAMPING_BOX_VARIANCE 1

	// Min max samples that are within distance from output pixel. 
	#define HISTORY_CLAMPING_BOX_SAMPLE_DISTANCE 2

	// Like the variance method, but uses Student-T distribution to model uncertainty in variance and mean.
	#define HISTORY_CLAMPING_BOX_T_INTERVAL 3

/** Qualities */
	#define TAA_QUALITY_LOW 0
	#define TAA_QUALITY_MEDIUM 1
	#define TAA_QUALITY_HIGH 2
	#define TAA_QUALITY_MEDIUM_HIGH 3

//------------------------------------------------------- CONFIGS

// Compute shaders always do responsive TAA in a single pass.
#if COMPUTESHADER
	#define AA_SINGLE_PASS_RESPONSIVE SHADING_PATH_DEFERRED //The StencilTexture is not available on the mobile platform
#endif

#if TAA_PASS_CONFIG == 0 // Main
	#if TAA_ALPHA_CHANNEL
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_TRANSLUCENCY)
	#else
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB)
	#endif
	#define AA_BICUBIC 1
	#define AA_CROSS 2
	#define AA_DYNAMIC 1
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_TONE 1
	#define AA_YCOCG 1

	#if TAA_QUALITY == TAA_QUALITY_LOW
		#define AA_FILTERED 0
		#define AA_DYNAMIC_ANTIGHOST 0

	#elif TAA_QUALITY == TAA_QUALITY_MEDIUM
		#define AA_FILTERED 1
		#define AA_DYNAMIC_ANTIGHOST 0

	#elif TAA_QUALITY == TAA_QUALITY_HIGH
		#define AA_FILTERED 1
		#define AA_DYNAMIC_ANTIGHOST 1

	#elif TAA_QUALITY == TAA_QUALITY_MEDIUM_HIGH
		#define AA_FILTERED 1
		#define AA_DYNAMIC_ANTIGHOST 1

	#else
		#error Unknown TAA quality
	#endif

	#if COMPUTESHADER
		#if AA_MOBILE_CONFIG
			#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_DISABLE) // the shared memory is not as efficient as expected on mobile devices, try not to use it on mobile devices.
		#else
			#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_LDS)
		#endif
	#endif
#elif TAA_PASS_CONFIG == 7 // Used for Hair
	#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_OPACITY)
	#define AA_DYNAMIC 1
	#define AA_FILTERED 0
	#define AA_LERP 3
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_YCOCG 0
#elif TAA_PASS_CONFIG == 3 // Used for SSR
	#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_OPACITY)
	#define AA_DYNAMIC 1
	#define AA_FILTERED 1
	#define AA_LERP 8
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_YCOCG 1

#elif TAA_PASS_CONFIG == 4 // Used for LightShaft
	#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_OPACITY)
	#define AA_FILTERED 1
	#define AA_LERP 64
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_YCOCG 1
	#define AA_LOWER_RESOLUTION 1

#elif TAA_PASS_CONFIG == 1 || TAA_PASS_CONFIG == 2 // MainUpsampling & MainSuperSampling
	#if TAA_ALPHA_CHANNEL
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_TRANSLUCENCY)
	#else
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB)
	#endif
	#define AA_BICUBIC 1
	#define AA_CROSS 1
	#define AA_DYNAMIC 1
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_TONE 1
	#if SWITCH_PROFILE || SWITCH_PROFILE_FORWARD
		#define AA_YCOCG 0
	#else
		#define AA_YCOCG 1
	#endif
	
	#define AA_UPSAMPLE 1
	#define AA_UPSAMPLE_ADAPTIVE_FILTERING 1

	#if TAA_QUALITY == TAA_QUALITY_LOW
		#define AA_FILTERED 0
		#if AA_MOBILE_CONFIG
			#define AA_SAMPLES 5
		#else
			#define AA_SAMPLES 6
		#endif

	#elif TAA_QUALITY == TAA_QUALITY_MEDIUM
		#define AA_FILTERED 1
		#if AA_MOBILE_CONFIG
			#define AA_SAMPLES 5
		#else
			#define AA_SAMPLES 6
		#endif

	#elif TAA_QUALITY == TAA_QUALITY_HIGH
		#define AA_HISTORY_CLAMPING_BOX (HISTORY_CLAMPING_BOX_SAMPLE_DISTANCE)
		#define AA_FILTERED 1
		#define AA_DYNAMIC_ANTIGHOST 1
		#define AA_SAMPLES 9

	#elif TAA_QUALITY == TAA_QUALITY_MEDIUM_HIGH
		#define AA_FILTERED 1
		#if AA_MOBILE_CONFIG
			#define AA_SAMPLES 5
		#else
			#define AA_SAMPLES 6
		#endif
		#define AA_DYNAMIC_ANTIGHOST 1

	#else
		#error Unknown TAA quality
	#endif

	#if COMPUTESHADER
		// Do not use LDS caching for screen percentage > 100% or < 50%.
		#if TAA_SCREEN_PERCENTAGE_RANGE == 2 || TAA_SCREEN_PERCENTAGE_RANGE == 3
			#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_VGPR_3X3)


		#else
			#if AA_MOBILE_CONFIG
				#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_DISABLE) // the shared memory is not as efficient as expected on mobile devices, try not to use it on mobile devices.
			#else
				#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_LDS)
			#endif
		#endif
	#endif
	
#elif TAA_PASS_CONFIG == 5 || TAA_PASS_CONFIG == 6 // Used for diaphragm DOF pre-filtering.
	#if TAA_ALPHA_CHANNEL
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_OPACITY_COC)
	#else
		#define AA_HISTORY_PAYLOAD (HISTORY_PAYLOAD_RGB_COC)
	#endif
	#define AA_CROSS 4 // because running at half res.
	#define AA_DYNAMIC 1
	#define AA_FORCE_ALPHA_CLAMP 1
	#define AA_MANUALLY_CLAMP_HISTORY_UV 1
	#define AA_LOWER_RESOLUTION 1

	#if TAA_PASS_CONFIG == 6
		#define AA_UPSAMPLE 1
		#define AA_UPSAMPLE_ADAPTIVE_FILTERING 1
	#else
		#define AA_UPSAMPLE 0
	#endif

	#if TAA_QUALITY == TAA_QUALITY_MEDIUM
		#if AA_UPSAMPLE
			#define AA_BICUBIC 1
			#define AA_FILTERED 1
		#endif

	#elif TAA_QUALITY == TAA_QUALITY_HIGH
		#define AA_BICUBIC 1
		#define AA_FILTERED 1
		#define AA_YCOCG 1
		#define AA_HISTORY_CLAMPING_BOX (HISTORY_CLAMPING_BOX_MIN_MAX)

	#elif TAA_QUALITY == TAA_QUALITY_MEDIUM_HIGH
		#if AA_UPSAMPLE
			#define AA_BICUBIC 1
			#define AA_FILTERED 1
		#endif

	#else
		#error Unknown TAA quality
	#endif

	#if TAA_SCREEN_PERCENTAGE_RANGE != 2 && AA_UPSAMPLE
		//#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_LDS) // TODO: doesn't work yet.
	#endif
	
#else
	#error Unknown TAA pass config. Have you changed ETAAPassConfig without updating me?

#endif

#if TAA_SCREEN_PERCENTAGE_RANGE == 2
	#define AA_DOWNSAMPLE 1
#else
	#define AA_DOWNSAMPLE 0
#endif


//------------------------------------------------------- CONFIG DISABLED DEFAULTS

// Num samples of current frame
#ifndef AA_SAMPLES
	#define AA_SAMPLES 5
#endif

// 1 = Use tighter AABB clamp for history.
// 0 = Use simple min/max clamp.
#ifndef AA_CLIP
	#define AA_CLIP 0
#endif

// Cross distance in pixels used in depth search X pattern.
// 0 = Turn this feature off.
// 2 = Is required for standard temporal AA pass.
#ifndef AA_CROSS
	#define AA_CROSS 0
#endif

// 1 = Use dynamic motion.
// 0 = Skip dynamic motion, currently required for half resolution passes.
#ifndef AA_DYNAMIC
	#define AA_DYNAMIC 0
#endif

// 0 = Dynamic motion based lerp value (default).
// non-zero = Use 1/LERP fixed lerp value (used for reflections).
#ifndef AA_LERP
	#define AA_LERP 0
#endif

// 1 = Use higher quality round clamp.
// 0 = Use lower quality but faster box clamp.
#ifndef AA_ROUND
	#define AA_ROUND 0
#endif

// Force clamp on alpha.
#ifndef AA_FORCE_ALPHA_CLAMP
	#define AA_FORCE_ALPHA_CLAMP 0
#endif

// Use YCoCg path.
#ifndef AA_YCOCG
	#define AA_YCOCG 0
#endif

// Bicubic filter history
#ifndef AA_BICUBIC
	#define AA_BICUBIC 0
#endif

// Tone map to kill fireflies
#ifndef AA_TONE
	#define AA_TONE 0
#endif

// Antighosting using dynamic mask
#ifndef AA_DYNAMIC_ANTIGHOST
	#define AA_DYNAMIC_ANTIGHOST 0
#endif

// Sample the stencil buffer inline rather than multiple masked passes.
#ifndef AA_SINGLE_PASS_RESPONSIVE
	#define AA_SINGLE_PASS_RESPONSIVE	0
#endif

// Upsample the output.
#ifndef AA_UPSAMPLE
	#define AA_UPSAMPLE 0
#endif

// Method used for generating the history clamping box.
#ifndef AA_HISTORY_CLAMPING_BOX
	#define AA_HISTORY_CLAMPING_BOX (HISTORY_CLAMPING_BOX_MIN_MAX)
#endif

// Change the upsampling filter size when history is rejected that reduce blocky output pixels.
#ifndef AA_UPSAMPLE_ADAPTIVE_FILTERING
	#define AA_UPSAMPLE_ADAPTIVE_FILTERING 0
#endif

// Whether this pass run at lower resolution than main view rectangle.
#ifndef AA_LOWER_RESOLUTION
	#define AA_LOWER_RESOLUTION 0
#endif

// Whether the history buffer UV should be manually clamped.
#ifndef AA_MANUALLY_CLAMP_HISTORY_UV
	#define AA_MANUALLY_CLAMP_HISTORY_UV 0
#endif


//------------------------------------------------------- CONFIG ENABLED DEFAULTS

// Always enable scene color filtering
// 1 = Use filtered sample.
// 0 = Use center sample.
#ifndef AA_FILTERED
	#define AA_FILTERED 1
#endif

// Always enable AA_NAN to avoid all NAN in all TAA pass that is more convenient considering the amount of / 0 we can have.
// 0 = Don't use.
// 1 = Use extra clamp to avoid NANs
#ifndef AA_NAN
	#define AA_NAN 1
#endif

// Neighborhood clamping. Disable for testing reprojection. Always enabled, well because TAA is totally broken otherwise.
#ifndef AA_CLAMP
	#define AA_CLAMP 1
#endif

// By default, always cache neighbooring samples into VGPR.
#ifndef AA_SAMPLE_CACHE_METHOD
	#if COMPUTESHADER
		#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_VGPR_3X3)
	#else
		#define AA_SAMPLE_CACHE_METHOD (AA_SAMPLE_CACHE_METHOD_DISABLE)
	#endif
#endif

// By default, enable stocastic quantization of the output.
#ifndef AA_ENABLE_STOCASTIC_QUANTIZATION
	#define AA_ENABLE_STOCASTIC_QUANTIZATION 1
#endif


//------------------------------------------------------- MENDATORY CONFIG

#ifndef AA_HISTORY_PAYLOAD
	#error You forgot to defines the history payload.
#endif


//------------------------------------------------------- DERIVES

// Defines number of component in history payload.
#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB
	#define HISTORY_PAYLOAD_COMPONENTS 3
#elif AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
	#define HISTORY_PAYLOAD_COMPONENTS 5
#else
	#define HISTORY_PAYLOAD_COMPONENTS 4
#endif

// Defines the number of render target to store TAA's history.
#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
	#define HISTORY_RENDER_TARGETS 2
#else
	#define HISTORY_RENDER_TARGETS 1
#endif


//------------------------------------------------------- CONFIG CHECKS

#if AA_SAMPLES != 9 && AA_SAMPLES != 5 && AA_SAMPLES != 6
	#error Samples must be 5, (6 for TAAU) or 9
#endif

#if AA_SAMPLE_CACHE_METHOD >= 2 && !COMPUTESHADER
	#error Group share only for compute shader.
#endif


//------------------------------------------------------- CONSTANTS

// K = Center of the nearest input pixel.
// O = Center of the output pixel.
//
//          |           |
//    0     |     1     |     2
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          | O         |
//    3     |     K     |     5
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          |           |
//    6     |     7     |     8
//          |           |
//
static const int2 kOffsets3x3[9] =
{
	int2(-1, -1),
	int2( 0, -1),
	int2( 1, -1),
	int2(-1,  0),
	int2( 0,  0), // K
	int2( 1,  0),
	int2(-1,  1),
	int2( 0,  1),
	int2( 1,  1),
};
	
// Indexes of the 3x3 square.
static const uint kSquareIndexes3x3[9] = { 0, 1, 2, 3, 4, 5, 6, 7, 8 };

// Indexes of the offsets to have plus + shape.
static const uint kPlusIndexes3x3[5] = { 1, 3, 4, 5, 7 };

// Number of neighbors.
static const uint kNeighborsCount = 9;


#if AA_UPSAMPLE
	// T = Center of the nearest top left pixel input pixel.
	// O = Center of the output pixel.
	//
	//          | 
	//    T     |     .
	//          | 
	//       O  | 
	//  --------+--------
	//          | 
	//          | 
	//    .     |     .
	//          | 
	static const int2 Offsets2x2[4] =
	{
		int2( 0,  0), // T
		int2( 1,  0),
		int2( 0,  1),
		int2( 1,  1),
	};
		
	// Indexes of the 2x2 square.
	static const uint SquareIndexes2x2[4] = { 0, 1, 2, 3 };

#endif // AA_UPSAMPLE


//------------------------------------------------------- PARAMETERS

float HistoryPreExposureCorrection;
float CurrentFrameWeight;
int bCameraCut;

DECLARE_SCALAR_ARRAY(float, SampleWeights, 9);
DECLARE_SCALAR_ARRAY(float, PlusWeights, 5);

float4 ViewportUVToInputBufferUV;
float4 MaxViewportUVAndSvPositionToViewportUV;

float2 ScreenPosAbsMax;
float4 ScreenPosToHistoryBufferUV;

float4 InputSceneColorSize;
int2 InputMinPixelCoord;
int2 InputMaxPixelCoord;
Texture2D InputSceneColor;
SamplerState InputSceneColorSampler;
Texture2D InputSceneMetadata;
SamplerState InputSceneMetadataSampler;

Texture2D SceneDepthTexture;
SamplerState SceneDepthTextureSampler;

#if COMPILER_GLSL_ES3_1
Texture2D<uint4> GBufferVelocityTextureSRV;
#else
Texture2D GBufferVelocityTexture;
SamplerState GBufferVelocityTextureSampler;
#endif

Texture2D<uint2> StencilTexture;

Texture2D HistoryBuffer_0;
Texture2D HistoryBuffer_1;
SamplerState HistoryBufferSampler_0;
SamplerState HistoryBufferSampler_1;
float4 HistoryBufferSize;
float4 HistoryBufferUVMinMax;

float CoCBilateralFilterStrength;

float4 OutputViewportSize;
float4 OutputViewportRect;
float3 OutputQuantizationError;

#if COMPUTESHADER
RWTexture2D<float4> OutComputeTex_0;

#if TAA_DOWNSAMPLE 
// This shader permutation outputs half resolution image in addition to main full-res one.
// It is more efficient than performing a separate downsampling pass afterwards.
RWTexture2D<float4> OutComputeTexDownsampled;
groupshared float4 GroupSharedDownsampleArray[THREADGROUP_SIZEX*THREADGROUP_SIZEY]; // TODO: share this with GroupSharedArrayF4 when possible
#endif // TAA_DOWNSAMPLE

#if HISTORY_RENDER_TARGETS == 2
	RWTexture2D<float4> OutComputeTex_1;
#endif // HISTORY_RENDER_TARGETS == 2
#endif

// Temporal upsample specific params.
#if AA_UPSAMPLE

float2 InputViewMin;
float4 InputViewSize;

// Temporal jitter at the pixel scale.
float2 TemporalJitterPixels;

float ScreenPercentage;
float UpscaleFactor; // = 1 / ScreenPercentage

#endif // AA_UPSAMPLE


//------------------------------------------------------- FUNCTIONS

#if COMPILER_SUPPORTS_HLSL2021

template<typename T>
void CorrectExposure(inout T X)
{
	X *= HistoryPreExposureCorrection;
}

#endif

ENCODED_VELOCITY_TYPE SampleVelocityTexture(float2 BufferUV, int2 PixelOffset = int2(0, 0))
{
#if COMPILER_GLSL_ES3_1
	int2 Coord = int2(BufferUV * InputSceneColorSize.xy) + PixelOffset;
	return GBufferVelocityTextureSRV.Load(int3(Coord, 0));
#else
	return GBufferVelocityTexture.SampleLevel(GBufferVelocityTextureSampler, BufferUV, 0, PixelOffset);
#endif
}

float3 RGBToYCoCg( float3 RGB )
{
	float Y  = dot( RGB, float3(  1, 2,  1 ) );
	float Co = dot( RGB, float3(  2, 0, -2 ) );
	float Cg = dot( RGB, float3( -1, 2, -1 ) );
	
	float3 YCoCg = float3( Y, Co, Cg );
	return YCoCg;
}

float3 YCoCgToRGB( float3 YCoCg )
{
	float Y  = YCoCg.x * 0.25;
	float Co = YCoCg.y * 0.25;
	float Cg = YCoCg.z * 0.25;

	float R = Y + Co - Cg;
	float G = Y + Cg;
	float B = Y - Co - Cg;

	float3 RGB = float3( R, G, B );
	return RGB;
}

// Faster but less accurate luma computation. 
// Luma includes a scaling by 4.
float Luma4(float3 Color)
{
	return (Color.g * 2.0) + (Color.r + Color.b);
}

// Optimized HDR weighting function.
float HdrWeight4(float3 Color, float Exposure) 
{
	return rcp(Luma4(Color) * Exposure + 4.0);
}

float HdrWeightY(float Color, float Exposure) 
{
	return rcp(Color * Exposure + 4.0);
}

// Intersect ray with AABB, knowing there is an intersection.
//   Dir = Ray direction.
//   Org = Start of the ray.
//   Box = Box is at {0,0,0} with this size.
// Returns distance on line segment.
float IntersectAABB(float3 Dir, float3 Org, float3 Box)
{
	#if PS4_PROFILE
		// This causes flicker, it should only be used on PS4 until proper fix is in.
		if(min(min(abs(Dir.x), abs(Dir.y)), abs(Dir.z)) < (1.0/65536.0)) return 1.0;
	#endif
	float3 RcpDir = rcp(Dir);
	float3 TNeg = (  Box  - Org) * RcpDir;
	float3 TPos = ((-Box) - Org) * RcpDir;
	return max(max(min(TNeg.x, TPos.x), min(TNeg.y, TPos.y)), min(TNeg.z, TPos.z));
}

float HistoryClip(float3 History, float3 Filtered, float3 NeighborMin, float3 NeighborMax)
{
#if 0
	float3 Min = min(Filtered, min(NeighborMin, NeighborMax));
	float3 Max = max(Filtered, max(NeighborMin, NeighborMax));	
	float3 Avg2 = Max + Min;
	float3 Dir = Filtered - History;
	float3 Org = History - Avg2 * 0.5;
	float3 Scale = Max - Avg2 * 0.5;
	return saturate(IntersectAABB(Dir, Org, Scale));
#else
	float3 BoxMin = NeighborMin;
	float3 BoxMax = NeighborMax;
	//float3 BoxMin = min( Filtered, NeighborMin );
	//float3 BoxMax = max( Filtered, NeighborMax );

	float3 RayOrigin = History;
	float3 RayDir = Filtered - History;
	RayDir = select(abs( RayDir ) < (1.0/65536.0), (1.0/65536.0), RayDir);
	float3 InvRayDir = rcp( RayDir );

	float3 MinIntersect = (BoxMin - RayOrigin) * InvRayDir;
	float3 MaxIntersect = (BoxMax - RayOrigin) * InvRayDir;
	float3 EnterIntersect = min( MinIntersect, MaxIntersect );
	return max3( EnterIntersect.x, EnterIntersect.y, EnterIntersect.z );
#endif
}

float2 WeightedLerpFactors(float WeightA, float WeightB, float Blend)
{
	float BlendA = (1.0 - Blend) * WeightA;
	float BlendB =        Blend  * WeightB;
	float RcpBlend = rcp(BlendA + BlendB);
	BlendA *= RcpBlend;
	BlendB *= RcpBlend;
	return float2(BlendA, BlendB);
}

#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC || AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC

// Computes the bilateral weight according to two Coc radii.
float ComputeBilateralWeight(float RefCocRadius, float SampleCocRadius)
{
	float Factor = (abs(RefCocRadius) > 1 ? rcp(abs(RefCocRadius)) : 1.0);
	return saturate(1 - abs(RefCocRadius - SampleCocRadius) * Factor);
}

float ComputeNeightborSampleBilateralWeight(float CenterCocRadius, float SampleCocRadius)
{
	float Factor = (abs(CenterCocRadius) > 1 ? rcp(abs(CenterCocRadius)) : 1.0);
	return saturate(1 - (CenterCocRadius - SampleCocRadius) * Factor);
}


#endif


//------------------------------------------------------- HISTORY's PAYLOAD

// Payload of the TAA's history.
struct FTAAHistoryPayload
{
	// Transformed scene color and alpha channel.
	float4 Color;

	// Radius of the circle of confusion for DOF.
	float CocRadius;
};

FTAAHistoryPayload MulPayload(in FTAAHistoryPayload Payload, in float x)
{
	Payload.Color *= x;
	Payload.CocRadius *= x;
	return Payload;
}

FTAAHistoryPayload AddPayload(in FTAAHistoryPayload Payload0, in FTAAHistoryPayload Payload1)
{
	Payload0.Color += Payload1.Color;
	Payload0.CocRadius += Payload1.CocRadius;
	return Payload0;
}

FTAAHistoryPayload MinPayload(in FTAAHistoryPayload Payload0, in FTAAHistoryPayload Payload1)
{
	Payload0.Color = min(Payload0.Color, Payload1.Color);
	Payload0.CocRadius = min(Payload0.CocRadius, Payload1.CocRadius);
	return Payload0;
}

FTAAHistoryPayload MaxPayload(in FTAAHistoryPayload Payload0, in FTAAHistoryPayload Payload1)
{
	Payload0.Color = max(Payload0.Color, Payload1.Color);
	Payload0.CocRadius = max(Payload0.CocRadius, Payload1.CocRadius);
	return Payload0;
}

FTAAHistoryPayload Pow2Payload(in FTAAHistoryPayload Payload)
{
	Payload.Color = Pow2(Payload.Color);
	Payload.CocRadius = Pow2(Payload.CocRadius);
	return Payload;
}

FTAAHistoryPayload MinPayload3(in FTAAHistoryPayload Payload0, in FTAAHistoryPayload Payload1, in FTAAHistoryPayload Payload2)
{
	Payload0.Color = min3(Payload0.Color, Payload1.Color, Payload2.Color);
	Payload0.CocRadius = min3(Payload0.CocRadius, Payload1.CocRadius, Payload2.CocRadius);
	return Payload0;
}

FTAAHistoryPayload MaxPayload3(in FTAAHistoryPayload Payload0, in FTAAHistoryPayload Payload1, in FTAAHistoryPayload Payload2)
{
	Payload0.Color = max3(Payload0.Color, Payload1.Color, Payload2.Color);
	Payload0.CocRadius = max3(Payload0.CocRadius, Payload1.CocRadius, Payload2.CocRadius);
	return Payload0;
}


//------------------------------------------------------- TAA INTERMEDIARY STRUCTURES

// Output pixel parameters. Should not be modified once setup.
struct FTAAInputParameters
{
	// Compute shader dispatch params, set to 0 in pixel shader.
	uint2 GroupId;
	uint2 GroupThreadId;
	uint GroupThreadIndex;

	// Viewport UV of the output pixel.
	float2 ViewportUV;

	// Position of the output pixel on screen.
	float2 ScreenPos;

	// Buffer UV of the nearest input pixel.
	float2 NearestBufferUV;

	#if AA_UPSAMPLE
		// Buffer UV of the nearest top left input pixel.
		float2 NearestTopLeftBufferUV;
	#endif
	
	// Whether this pixel should be responsive.
	float bIsResponsiveAAPixel;

	// Frame exposure's scale.
	float FrameExposureScale;
	
	// Cache of neightbors' transformed scene color.
	#if AA_SAMPLE_CACHE_METHOD == AA_SAMPLE_CACHE_METHOD_VGPR_3X3
		float4 CachedNeighbors0[kNeighborsCount];

		#if HISTORY_RENDER_TARGETS == 2
			float CachedNeighbors1[kNeighborsCount];
		#endif
	#endif
};


// Intermediary results for major function to conveniently share values between them.
//
// it is allowed to passdown this to major function with still unitialized variables.
struct FTAAIntermediaryResult
{
	// The filtered input.
	FTAAHistoryPayload Filtered;

	// Temporal weight of the filtered input.
	float FilteredTemporalWeight;

	// 1 / filtering kernel scale factor for AA_UPSAMPLE_ADAPTIVE_FILTERING.
	float InvFilterScaleFactor;
};


// Create intermediary result.
FTAAIntermediaryResult CreateIntermediaryResult()
{
	// Disable warning X4008: floating point division by zero
	#pragma warning(disable:4008)
	FTAAIntermediaryResult IntermediaryResult = (FTAAIntermediaryResult) (1.0 / 0.0);
	#pragma warning(default:4008)

	IntermediaryResult.FilteredTemporalWeight = 1;
	IntermediaryResult.InvFilterScaleFactor = 1;
	return IntermediaryResult;
}


// Transformed scene color's data for a sample.
struct FTAASceneColorSample
{
	// Transformed scene color and alpha channel.
	float4 Color;

	// Radius of the circle of confusion for DOF.
	float CocRadius;

	// HDR weight of the scene color sample.
	float HdrWeight;
};


//------------------------------------------------------- SCENE COLOR SPACE MANAGMENT

// Transform RAW linear scene color RGB to TAA's working color space.
float4 TransformSceneColor(float4 RawLinearSceneColorRGBA)
{
	#if AA_YCOCG
		return float4(RGBToYCoCg(RawLinearSceneColorRGBA.rgb), RawLinearSceneColorRGBA.a);
	#else
		return RawLinearSceneColorRGBA;
	#endif
}

// Reciprocal of TransformSceneColor().
float4 TransformBackToRawLinearSceneColor(float4 SceneColor)
{
	#if AA_YCOCG
		return float4(YCoCgToRGB(SceneColor.xyz), SceneColor.a);
	#else
		return SceneColor;
	#endif
}

// Transform current frame's RAW scene color RGB to TAA's working color space.
float4 TransformCurrentFrameSceneColor(float4 RawSceneColorRGBA)
{
	return TransformSceneColor(RawSceneColorRGBA);
}

// Get the Luma4 of the sceneColor
float GetSceneColorLuma4(float4 SceneColor)
{
	#if AA_YCOCG
		return SceneColor.x;
	#else
		return Luma4(SceneColor.rgb);
	#endif
}

// Get the HDR weight of the transform scene color.
float GetSceneColorHdrWeight(
	in FTAAInputParameters InputParams,
	float4 SceneColor)
{
	#if AA_YCOCG
		return HdrWeightY(SceneColor.x, InputParams.FrameExposureScale);
	#else
		return HdrWeight4(SceneColor.rgb, InputParams.FrameExposureScale);
	#endif
}


//------------------------------------------------------- INPUT SAMPLE CACHING.
// API to sample input scene color and depth through caching system.
// 
// Precache scene color or depth:
//		PrecacheInputSceneColor(InputParams);
//		PrecacheInputSceneDepth(InputParams);
//
// Then sample scene color or depth:
//		SampleCachedSceneColorTexture(InputParams, /* Offset = */ int2(-1, -1));
//		SampleCachedSceneDepthTexture(InputParams, /* Offset = */ int2(-1, -1));
//
//		<Offset> parameter is meant to be compile time constant of the pixel offset from nearest input sample.

#if AA_SAMPLE_CACHE_METHOD == AA_SAMPLE_CACHE_METHOD_VGPR_3X3
//------------------------------------------------------- 3x3 NEIGHTBORS CACHING

	#define AA_PRECACHE_SCENE_COLOR 1
	
	void PrecacheInputSceneColor(inout FTAAInputParameters InputParams)
	{
		// Precache 3x3 input scene color into FTAAInputParameters::CachedNeighbors.
		UNROLL
		for (uint i = 0; i < kNeighborsCount; i++)
		{
			int2 Coord = int2(InputParams.NearestBufferUV * InputSceneColorSize.xy) + kOffsets3x3[i];
			Coord = clamp(Coord, InputMinPixelCoord, InputMaxPixelCoord);

			InputParams.CachedNeighbors0[i] = TransformCurrentFrameSceneColor(InputSceneColor[Coord]);
	
			#if HISTORY_RENDER_TARGETS == 2
				InputParams.CachedNeighbors1[i] = InputSceneMetadata[Coord].r;
			#endif
		}
	}

	FTAASceneColorSample SampleCachedSceneColorTexture(
		inout FTAAInputParameters InputParams,
		int2 PixelOffset)
	{
		// PixelOffset is const at compile time. Therefore all this computaton is actually free.
		uint NeighborsId = uint(4 + PixelOffset.x + PixelOffset.y * 3);
		FTAASceneColorSample Sample;

		Sample.Color = InputParams.CachedNeighbors0[NeighborsId];
		Sample.CocRadius = 0;

		#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC
			Sample.CocRadius = Sample.Color.a;
		#elif AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
			Sample.CocRadius = InputParams.CachedNeighbors1[NeighborsId];
		#endif

		Sample.HdrWeight = GetSceneColorHdrWeight(InputParams, Sample.Color);
		return Sample;
	}


#elif AA_SAMPLE_CACHE_METHOD == AA_SAMPLE_CACHE_METHOD_LDS
//-------------------------------- Generic LDS functions.

// Returns SV_GroupIndex.
uint GetGroupIndex(in FTAAInputParameters InputParams)
{
	if (0)
	{
		// Group thread index doesn't actually suppress ALU instruction on consoles.
		return InputParams.GroupThreadIndex;
	}
	return InputParams.GroupThreadId.x + InputParams.GroupThreadId.y * THREADGROUP_SIZEX;
}


//------------------------------------------------------- COMPUTE SHADER GROUP SHARE OPTIMIZATION

#if THREADGROUP_SIZEX != THREADGROUP_SIZEY
	#error AA_SAMPLE_CACHE_METHOD >= 2 assume square shaped tiles.
#endif

#ifndef TAA_SCREEN_PERCENTAGE_RANGE
	#error LDS cache needs to know the screen percentage range.
#endif

#define AA_PRECACHE_IMPLEMENTATIONS 1

// Total number of thread per group.
#define THREADGROUP_TOTAL (THREADGROUP_SIZEX * THREADGROUP_SIZEY)

// Configure base width of the LDS tile according to screen percentage range.
// If screen percentage is lower than 75%, we can set the base tile width to 6, which will subsequently create a scene color
// tile width of 8 that can be cache with only one iteration per lane in PrecacheInputSceneColorToLDS(), and reduce size(LDS) = 1k.
#if TAA_SCREEN_PERCENTAGE_RANGE == 1 // screen percentage < 75%
	#define LDS_BASE_TILE_WIDTH 6
#elif TAA_SCREEN_PERCENTAGE_RANGE == 0 // screen percentage in [75%; 100%]
	#define LDS_BASE_TILE_WIDTH THREADGROUP_SIZEX
#elif TAA_SCREEN_PERCENTAGE_RANGE == 2 // screen percentage > 100%
	#error Should not use LDS caching.
#else
	#error Unknown screen percentage range.
#endif


//-------------------------------- Configuration.

// Configuration of what should be prefetched.
// 1: use Load; 2: use gather4. 

#if !AA_UPSAMPLE
// Disables scene depth caching for TAA upsample because the extra screen percentage ALU is making things worst.
#define AA_PRECACHE_SCENE_DEPTH 2
#endif

// 1: use load.
#define AA_PRECACHE_SCENE_COLOR 1

// Precache GetSceneColorHdrWeight() into scene color's alpha channel.
#define AA_PRECACHE_SCENE_HDR_WEIGHT (AA_TONE && HISTORY_PAYLOAD_COMPONENTS == 3)

// Layout of scene color in LDS.
// 0: AoS
// 1: SoA
// 2: AoSoA (SoA scene color, AoS GetSceneColorHdrWeight())
#define LDS_COLOR_LAYOUT 0


//-------------------------------- Depth tile constants.

// Number of texels arround the group tile for depth.
#define LDS_DEPTH_TILE_BORDER_SIZE (AA_CROSS)

// Width in texels of the depth tile cached into LDS.
#define LDS_DEPTH_TILE_WIDTH (LDS_BASE_TILE_WIDTH + 2 * LDS_DEPTH_TILE_BORDER_SIZE)

// Total number of texels cached in the depth tile.
#define LDS_DEPTH_ARRAY_SIZE (LDS_DEPTH_TILE_WIDTH * LDS_DEPTH_TILE_WIDTH)


//-------------------------------- Scene color tile constants.

// TODO: shader permutation for screen percentage <= 75% with AA_UPSAMPLE to fit in 1k LDS.

// Number of scene color component that gets cached.
#if HISTORY_PAYLOAD_COMPONENTS == 4 || AA_PRECACHE_SCENE_HDR_WEIGHT
#define LDS_COLOR_COMPONENT_COUNT 4
#else
#define LDS_COLOR_COMPONENT_COUNT 3
#endif

// Number of texels arround the group tile for scene color.
#define LDS_COLOR_TILE_BORDER_SIZE (1)

// Width in texels of the depth tile cached into LDS.
#define LDS_COLOR_TILE_WIDTH (LDS_BASE_TILE_WIDTH + 2 * LDS_COLOR_TILE_BORDER_SIZE)

// Total number of texels cached in the scene color tile.
#define LDS_COLOR_ARRAY_SIZE (LDS_COLOR_TILE_WIDTH * LDS_COLOR_TILE_WIDTH)


//-------------------------------- Group shared global.

// Size of the LDS to be allocated.
#define LDS_ARRAY_SIZE (LDS_COLOR_ARRAY_SIZE * LDS_COLOR_COMPONENT_COUNT)
#if LDS_ARRAY_SIZE < LDS_DEPTH_ARRAY_SIZE
	#error LDS_ARRAY_SIZE assumed scene color caching is bigger than scene depth caching.
#endif

// Some compilers may have issues optimising LDS store instructions, therefore we give the compiler a hint by using a float4 LDS.
#if defined(AA_PRECACHE_SCENE_DEPTH)
#define LDS_USE_FLOAT4_ARRAY 0
#else
#define LDS_USE_FLOAT4_ARRAY (LDS_COLOR_COMPONENT_COUNT == 4 && LDS_COLOR_LAYOUT == 0)
#endif

#if LDS_USE_FLOAT4_ARRAY
groupshared float4 GroupSharedArrayF4[LDS_ARRAY_SIZE/4];
#else
groupshared float GroupSharedArray[LDS_ARRAY_SIZE];
#endif


//-------------------------------- Generic LDS tile functions.

#if AA_UPSAMPLE

// Get the pixel coordinate of the nearest input pixel K for group's thread 0.
float2 GetGroupThread0InputPixelCoord(in FTAAInputParameters InputParams)
{
	// Output pixel center position of the group thread index 0, relative to top left corner of the viewport.
	float2 Thread0SvPosition = InputParams.GroupId * uint2(THREADGROUP_SIZEX, THREADGROUP_SIZEY) + 0.5;
		
	// Output pixel's viewport UV group thread index 0.
	float2 Thread0ViewportUV = Thread0SvPosition * OutputViewportSize.zw;

	// Pixel coordinate of the center of output pixel O in the input viewport.
	float2 Thread0PPCo = Thread0ViewportUV * InputViewSize.xy + TemporalJitterPixels;
	
	// Pixel coordinate of the center of the nearest input pixel K.
	float2 Thread0PPCk = floor(Thread0PPCo) + 0.5;
		
	return InputViewMin.xy + Thread0PPCk;
}

#endif

// Get the texel offset of a LDS tile's top left corner.
uint2 GetGroupTileTexelOffset(in FTAAInputParameters InputParams, uint TileBorderSize)
{
	#if AA_UPSAMPLE
	{
		// Pixel coordinate of the center of the nearest input pixel K.
		float2 Thread0PPCk = GetGroupThread0InputPixelCoord(InputParams);
		
		return uint2(floor(Thread0PPCk) - TileBorderSize);
	}
	#else // !AA_UPSAMPLE
	{
		return OutputViewportRect.xy + InputParams.GroupId * uint2(THREADGROUP_SIZEX, THREADGROUP_SIZEY) - TileBorderSize;
	}
	#endif
}

// Get the index within the LDS array.
uint GetTileArrayIndexFromPixelOffset(in FTAAInputParameters InputParams, int2 PixelOffset, uint TileBorderSize)
{
	#if AA_UPSAMPLE
	{
		const float2 RowMultiplier = float2(1, TileBorderSize * 2 + LDS_BASE_TILE_WIDTH);

		float2 Thread0PPCk = GetGroupThread0InputPixelCoord(InputParams);
		float2 PPCk = InputParams.NearestBufferUV * InputSceneColorSize.xy;

		float2 TilePos = floor(PPCk) - floor(Thread0PPCk);
		return uint(dot(TilePos, RowMultiplier) + dot(float2(PixelOffset) + float(TileBorderSize), RowMultiplier));
	}
	#else
	{
		uint2 TilePos = InputParams.GroupThreadId + uint2(PixelOffset + TileBorderSize);
		return TilePos.x + TilePos.y * (TileBorderSize * 2 + LDS_BASE_TILE_WIDTH);
	}
	#endif
}


//-------------------------------- Share depth texture fetches.
#if defined(AA_PRECACHE_SCENE_DEPTH)

// Precache input scene depth into LDS.
void PrecacheInputSceneDepthToLDS(in FTAAInputParameters InputParams)
{
	uint2 GroupTexelOffset = GetGroupTileTexelOffset(InputParams, LDS_DEPTH_TILE_BORDER_SIZE);
	
	#if AA_PRECACHE_SCENE_DEPTH == 1
	// Prefetch depth buffer using Load.
	{
		const uint LoadCount = (LDS_DEPTH_ARRAY_SIZE + THREADGROUP_TOTAL - 1) / THREADGROUP_TOTAL;
		
		uint LinearGroupThreadId = GetGroupIndex(InputParams);

		UNROLL
		for (uint i = 0; i < LoadCount; i++)
		{
			uint2 TexelLocation =  GroupTexelOffset + uint2(
				LinearGroupThreadId % LDS_DEPTH_TILE_WIDTH,
				LinearGroupThreadId / LDS_DEPTH_TILE_WIDTH);

			if ((LinearGroupThreadId < LDS_DEPTH_ARRAY_SIZE) ||
				(i != LoadCount - 1) ||
				(LDS_DEPTH_ARRAY_SIZE % THREADGROUP_TOTAL) == 0)
			{
				GroupSharedArray[LinearGroupThreadId] = SceneDepthTexture.Load(uint3(TexelLocation, 0)).x;
			}
			LinearGroupThreadId += THREADGROUP_TOTAL;
		}

	}

	#elif AA_PRECACHE_SCENE_DEPTH == 2
	// Prefetch depth buffer using Gather.
	{
		const uint LoadCount = (LDS_DEPTH_ARRAY_SIZE / 4 + THREADGROUP_TOTAL - 1) / THREADGROUP_TOTAL;
		
		uint LinearGroupThreadId = GetGroupIndex(InputParams);
	
		UNROLL
		for (uint i = 0; i < LoadCount; i++)
		{
			uint2 TileDest = uint2(
				(2 * LinearGroupThreadId) % LDS_DEPTH_TILE_WIDTH,
				2 * ((2 * LinearGroupThreadId) / LDS_DEPTH_TILE_WIDTH));

			uint2 TexelLocation =  GroupTexelOffset + TileDest;

			uint DestI = TileDest.x + TileDest.y * LDS_DEPTH_TILE_WIDTH;

			if ((DestI < LDS_DEPTH_ARRAY_SIZE) ||
				(i != LoadCount - 1) ||
				((LDS_DEPTH_ARRAY_SIZE / 4) % THREADGROUP_TOTAL) == 0)
			{
				float2 UV = float2(TexelLocation + 0.5) * InputSceneColorSize.zw;
				float4 Depth = SceneDepthTexture.Gather(SceneDepthTextureSampler, UV);
				GroupSharedArray[DestI + 1 * LDS_DEPTH_TILE_WIDTH + 0] = Depth.x;
				GroupSharedArray[DestI + 1 * LDS_DEPTH_TILE_WIDTH + 1] = Depth.y;
				GroupSharedArray[DestI + 0 * LDS_DEPTH_TILE_WIDTH + 1] = Depth.z;
				GroupSharedArray[DestI + 0 * LDS_DEPTH_TILE_WIDTH + 0] = Depth.w;
			}
			LinearGroupThreadId += THREADGROUP_TOTAL;
		}
	}

	#else
		#error Wrong AA_PRECACHE_SCENE_DEPTH

	#endif
}

float SampleCachedSceneDepthTexture(in FTAAInputParameters InputParams, int2 PixelOffset)
{
	return GroupSharedArray[GetTileArrayIndexFromPixelOffset(InputParams, PixelOffset, LDS_DEPTH_TILE_BORDER_SIZE)];
}


#endif // define(AA_PRECACHE_SCENE_DEPTH)


//-------------------------------- Share color texture fetches.

#if defined(AA_PRECACHE_SCENE_COLOR)

// Return the index GroupSharedArray from a given ArrayIndex and ComponentId.
uint GetSceneColorLDSIndex(uint ArrayIndex, uint ComponentId)
{
	#if LDS_COLOR_LAYOUT == 0 // AoS
		return ArrayIndex * LDS_COLOR_COMPONENT_COUNT + ComponentId;
	#elif LDS_COLOR_LAYOUT == 1 // SoA
		return ArrayIndex + ComponentId * LDS_COLOR_ARRAY_SIZE;
	#else
		#error Unknown color layout.
	#endif
}

// Precache input scene color into LDS.
void PrecacheInputSceneColorToLDS(in FTAAInputParameters InputParams)
{
	const uint LoadCount = (LDS_COLOR_ARRAY_SIZE + THREADGROUP_TOTAL - 1) / THREADGROUP_TOTAL;
	
	#define LDS_FLOAT_UV AA_UPSAMPLE

	#if LDS_FLOAT_UV
		float LinearGroupThreadId = float(GetGroupIndex(InputParams));
		float2 Thread0PPCk = GetGroupThread0InputPixelCoord(InputParams);
		float2 GroupTexelOffset = Thread0PPCk - LDS_COLOR_TILE_BORDER_SIZE;
	#else
		uint LinearGroupThreadId = GetGroupIndex(InputParams);
		uint2 GroupTexelOffset = GetGroupTileTexelOffset(InputParams, LDS_COLOR_TILE_BORDER_SIZE);
	#endif
	
	UNROLL
	for (uint i = 0; i < LoadCount; i++)
	{
		#if LDS_FLOAT_UV
			float Y = floor(LinearGroupThreadId * (1.0 / LDS_COLOR_TILE_WIDTH));
			float X = LinearGroupThreadId - LDS_COLOR_TILE_WIDTH * Y;

			float2 TexelLocation = GroupTexelOffset + float2(X, Y);
		#else
			uint2 TexelLocation =  GroupTexelOffset + uint2(
				LinearGroupThreadId % LDS_COLOR_TILE_WIDTH,
				LinearGroupThreadId / LDS_COLOR_TILE_WIDTH);
		#endif

		if ((LinearGroupThreadId < LDS_COLOR_ARRAY_SIZE) ||
			(i != LoadCount - 1) ||
			(LDS_COLOR_ARRAY_SIZE % THREADGROUP_TOTAL) == 0)
		{
			#if LDS_FLOAT_UV
				int2 Coord = TexelLocation;
				Coord = clamp(Coord, InputMinPixelCoord, InputMaxPixelCoord);

				float4 RawColor = InputSceneColor[Coord];
			#else
				int2 Coord = int2(TexelLocation);
				Coord = clamp(Coord, InputMinPixelCoord, InputMaxPixelCoord);
				float4 RawColor = InputSceneColor.Load(uint3(Coord, 0));
			#endif

			float4 Color = TransformCurrentFrameSceneColor(RawColor);

			// Precache scene color's HDR weight into alpha channel to reduce rcp() instructions in innerloops.
			#if AA_PRECACHE_SCENE_HDR_WEIGHT
				Color.a = GetSceneColorHdrWeight(InputParams, Color);
			#endif

			#if LDS_USE_FLOAT4_ARRAY
				GroupSharedArrayF4[uint(LinearGroupThreadId)] = Color;

			#else
				GroupSharedArray[GetSceneColorLDSIndex(uint(LinearGroupThreadId), 0)] = Color.r;
				GroupSharedArray[GetSceneColorLDSIndex(uint(LinearGroupThreadId), 1)] = Color.g;
				GroupSharedArray[GetSceneColorLDSIndex(uint(LinearGroupThreadId), 2)] = Color.b;
				#if LDS_COLOR_COMPONENT_COUNT == 4
					GroupSharedArray[GetSceneColorLDSIndex(uint(LinearGroupThreadId), 3)] = Color.a;
				#endif
			
			#endif
		}
		LinearGroupThreadId += THREADGROUP_TOTAL;
	}
}

FTAASceneColorSample SampleCachedSceneColorTexture(
	in FTAAInputParameters InputParams,
	int2 PixelOffset)
{
	uint ArrayPos = GetTileArrayIndexFromPixelOffset(InputParams, PixelOffset, LDS_COLOR_TILE_BORDER_SIZE);
	
	FTAASceneColorSample Sample;
	Sample.CocRadius = 0;
	
	#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC || AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
		#error Unsupported history payload with LDS scene color caching.Test
	#endif

	#if LDS_USE_FLOAT4_ARRAY
		Sample.Color = GroupSharedArrayF4[ArrayPos];

		#if AA_PRECACHE_SCENE_HDR_WEIGHT
			Sample.HdrWeight = Sample.Color.a;
			Sample.Color.a = 0;
		#elif HISTORY_PAYLOAD_COMPONENTS != 4
			#error LDS_USE_FLOAT4_ARRAY assumes 4 components.
		#endif

	#else
		Sample.Color.r = GroupSharedArray[GetSceneColorLDSIndex(ArrayPos, 0)];
		Sample.Color.g = GroupSharedArray[GetSceneColorLDSIndex(ArrayPos, 1)];
		Sample.Color.b = GroupSharedArray[GetSceneColorLDSIndex(ArrayPos, 2)];
		Sample.Color.a = 0;
	
		#if HISTORY_PAYLOAD_COMPONENTS == 4
			Sample.Color.a = GroupSharedArray[GetSceneColorLDSIndex(ArrayPos, 3)];
		#elif AA_PRECACHE_SCENE_HDR_WEIGHT
			Sample.HdrWeight = GroupSharedArray[GetSceneColorLDSIndex(ArrayPos, 3)];
		#endif

	#endif
	
	// if scene color weight was not precached in LDS, compute it.
	#if !AA_PRECACHE_SCENE_HDR_WEIGHT
		Sample.HdrWeight = GetSceneColorHdrWeight(InputParams, Sample.Color);
	#endif
	
	// Color has already been transformed in PrecacheInputSceneColor.
	return Sample;
}


#endif // defined(AA_PRECACHE_SCENE_COLOR)


void PrecacheInputSceneDepth(in FTAAInputParameters InputParams)
{
	#if defined(AA_PRECACHE_SCENE_DEPTH)
		PrecacheInputSceneDepthToLDS(InputParams);
		GroupMemoryBarrierWithGroupSync();
	#endif
}

void PrecacheInputSceneColor(in FTAAInputParameters InputParams)
{
	#if defined(AA_PRECACHE_SCENE_DEPTH) && defined(AA_PRECACHE_SCENE_COLOR)
		GroupMemoryBarrierWithGroupSync();
	#endif
	#if defined(AA_PRECACHE_SCENE_COLOR)
		PrecacheInputSceneColorToLDS(InputParams);
		GroupMemoryBarrierWithGroupSync();
	#endif
}


#endif // AA_SAMPLE_CACHE_METHOD == AA_SAMPLE_CACHE_METHOD_LDS

//------------------------------------------------------- FALLBACK TO NO CACHING IMPLEMENTATIONS.

#if !defined(AA_PRECACHE_SCENE_DEPTH)

#if !defined(AA_PRECACHE_IMPLEMENTATIONS)

// Silently do no scene depth precaching.
void PrecacheInputSceneDepth(in FTAAInputParameters InputParams)
{ }

#endif


// Sample scene color.
float SampleCachedSceneDepthTexture(in FTAAInputParameters InputParams, int2 PixelOffset)
{
	return SceneDepthTexture.SampleLevel(SceneDepthTextureSampler, InputParams.NearestBufferUV, 0, PixelOffset).r;
}


#endif

#if !defined(AA_PRECACHE_SCENE_COLOR)

#if !defined(AA_PRECACHE_IMPLEMENTATIONS)

// Silently do no scene color precaching.
void PrecacheInputSceneColor(in FTAAInputParameters InputParams)
{ }

#endif


// Sample and transform scene color.
FTAASceneColorSample SampleCachedSceneColorTexture(
	in FTAAInputParameters InputParams,
	int2 PixelOffset)
{
	FTAASceneColorSample Sample;


	int2 Coord = int2(InputParams.NearestBufferUV * InputSceneColorSize.xy) + PixelOffset;
	Coord = clamp(Coord, InputMinPixelCoord, InputMaxPixelCoord);

	Sample.Color = TransformCurrentFrameSceneColor(InputSceneColor[Coord]);

	Sample.CocRadius = 0;

	#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC
		Sample.CocRadius = Sample.Color.a;
	#elif AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
		Sample.CocRadius = InputSceneMetadata[Coord].r;
	#endif

	Sample.HdrWeight = GetSceneColorHdrWeight(InputParams, Sample.Color);
	return Sample;
}


#endif


//------------------------------------------------------- TEMPORAL UPSAMPLING

#if AA_UPSAMPLE

// Returns the weight of a pixels at a coordinate <PixelDelta> from the PDF highest point.
float ComputeSampleWeigth(in FTAAIntermediaryResult IntermediaryResult, float2 PixelDelta)
{
	float u2 = UpscaleFactor * UpscaleFactor;

	// The point of InvFilterScaleFactor is to blur current frame scene color when upscaling.
	// Therefore there is no need to do it when downscaling.
	if (!AA_DOWNSAMPLE)
	{
		u2 *= (IntermediaryResult.InvFilterScaleFactor * IntermediaryResult.InvFilterScaleFactor);
	}

	#if 1
		// 1 - 1.9 * x^2 + 0.9 * x^4
		float x2 = saturate(u2 * dot(PixelDelta, PixelDelta));
		return (0.905 * x2 - 1.9) * x2 + 1;

	#else
		// original e ^ (- x^2 / (2 * s^2))
		const float Sigma = 0.47;
		const float ExponentInputFactor = (-0.5 / (Sigma * Sigma));
		
		float x2 = dot(PixelDelta, PixelDelta) * u2;
		return exp(ExponentInputFactor * x2);

	#endif
}


// Returns the weight of a pixels at a coordinate <PixelDelta> from the PDF highest point.
float ComputePixelWeigth(in FTAAIntermediaryResult IntermediaryResult, float2 PixelDelta)
{
	float u2 = UpscaleFactor * UpscaleFactor;

	// The point of InvFilterScaleFactor is to blur current frame scene color when upscaling.
	// Therefore there is no need to do it when downscaling.
	if (!AA_DOWNSAMPLE)
	{
		u2 *= (IntermediaryResult.InvFilterScaleFactor * IntermediaryResult.InvFilterScaleFactor);
	}

	#if 1
		// 1 - 1.9 * x^2 + 0.9 * x^4
		float x2 = saturate(u2 * dot(PixelDelta, PixelDelta));
		float r = (0.905 * x2 - 1.9) * x2 + 1;

	#else
		// original e ^ (- x^2 / (2 * s^2))
		const float Sigma = 0.47;
		const float ExponentInputFactor = (-0.5 / (Sigma * Sigma));
		
		float x2 = dot(PixelDelta, PixelDelta);
		float r = exp(ExponentInputFactor * x2);

	#endif

	if (!AA_DOWNSAMPLE)
	{
		// Multiply pixel weight ^ 2 by upscale factor because have only a probability = screen percentage ^ 2 to return 1.
		// Only do it for upsampling to not converge slower than if doing screen percentage 100%.
		return u2 * r;
	}
	return r;
}

#endif // AA_UPSAMPLE


//------------------------------------------------------- TAA MAJOR FUNCTIONS

// Filter input pixels.
void FilterCurrentFrameInputSamples(
	in FTAAInputParameters InputParams,
	inout FTAAIntermediaryResult IntermediaryResult)
{
	#if !AA_FILTERED
	{
		IntermediaryResult.Filtered.Color = SampleCachedSceneColorTexture(InputParams, int2(0, 0)).Color;
		IntermediaryResult.Filtered.CocRadius = SampleCachedSceneColorTexture(InputParams, int2(0, 0)).CocRadius;
		return;
	}
	#endif

	FTAAHistoryPayload Filtered;

	{
		#if AA_UPSAMPLE
			// Pixel coordinate of the center of output pixel O in the input viewport.
			float2 PPCo = InputParams.ViewportUV * InputViewSize.xy + TemporalJitterPixels;

			// Pixel coordinate of the center of the nearest input pixel K.
			float2 PPCk = floor(PPCo) + 0.5;
		
			// Vector in pixel between pixel K -> O.
			float2 dKO = PPCo - PPCk;
		#endif
		
		#if AA_SAMPLES == 9
			const uint SampleIndexes[9] = kSquareIndexes3x3;
		#elif AA_SAMPLES == 5 || AA_SAMPLES == 6
			const uint SampleIndexes[5] = kPlusIndexes3x3;
		#endif

		#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC || AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
			// Fetches center pixel's Coc for the bilateral filtering.
			float CenterCocRadius = SampleCachedSceneColorTexture(InputParams, int2(0, 0)).CocRadius;
		#endif

		float NeighborsHdrWeight = 0;
		float NeighborsFinalWeight = 0;
		float4 NeighborsColor = 0;

		UNROLL
		for (uint i = 0; i < AA_SAMPLES; i++)
		{
			// Get the sample offset from the nearest input pixel.
			int2 SampleOffset;
			
			#if AA_UPSAMPLE && AA_SAMPLES == 6
				if (i == 5)
				{
					SampleOffset = SignFastInt(dKO);
				}
				else
			#endif
			{
				const uint SampleIndex = SampleIndexes[i];
				SampleOffset = kOffsets3x3[SampleIndex];
			}
			float2 fSampleOffset = float2(SampleOffset);
			
			// When doing Coc bilateral, the center sample is accumulated last.
			#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC && 0
				if (all(SampleOffset == 0) && (AA_SAMPLES != 6 || i != 5))
				{
					continue;
				}
			#endif

			// Finds out the spatial weight of this input sample.
			#if AA_UPSAMPLE
				// Compute the pixel delta between output pixels and input pixel I.
				//  Note: abs() is unecessary because of the dot(dPP, dPP) latter on.
				float2 dPP = fSampleOffset - dKO;

				float SampleSpatialWeight = ComputeSampleWeigth(IntermediaryResult, dPP);

			#elif AA_SAMPLES == 9
				float SampleSpatialWeight = GET_SCALAR_ARRAY_ELEMENT(SampleWeights, i);

			#elif AA_SAMPLES == 5
				float SampleSpatialWeight = GET_SCALAR_ARRAY_ELEMENT(PlusWeights, i);

			#else
				#error Do not know how to compute filtering sample weight.

			#endif

			// Fetch sample.
			FTAASceneColorSample Sample = SampleCachedSceneColorTexture(InputParams, SampleOffset);
				
			// Finds out the sample's HDR weight.
			#if AA_TONE
				float SampleHdrWeight = Sample.HdrWeight;
			#else
				float SampleHdrWeight = 1;
			#endif

			// Finds out the sample's bilateral weight according to the payload.
			#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC
				float BilateralWeight = ComputeNeightborSampleBilateralWeight(CenterCocRadius, Sample.CocRadius);

			#else
				float BilateralWeight = 1;

			#endif

			float SampleFinalWeight = SampleSpatialWeight * SampleHdrWeight * BilateralWeight;

			// Apply pixel.
			NeighborsColor       += SampleFinalWeight * Sample.Color;
			NeighborsFinalWeight += SampleFinalWeight;

			NeighborsHdrWeight   += SampleSpatialWeight * SampleHdrWeight;
		}
		
		#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC || AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
		{
			FTAASceneColorSample Sample = SampleCachedSceneColorTexture(InputParams, 0);
			
			#if AA_UPSAMPLE
				float SampleWeight = ComputeSampleWeigth(IntermediaryResult, -dKO);

			#elif AA_SAMPLES == 9
				float SampleWeight = GET_SCALAR_ARRAY_ELEMENT(SampleWeights, 4);

			#else
				float SampleWeight = GET_SCALAR_ARRAY_ELEMENT(PlusWeights, 2);

			#endif
			
			if (AA_TONE)
			{
				SampleWeight *= Sample.HdrWeight;
			}

			// TODO: it feels wrong...
			//Filtered = NeighborsColor * (NeighborsHdrWeight * rcp(NeighborsFinalWeight)) + Sample.Color * SampleWeight;
			Filtered.Color = NeighborsColor * rcp(NeighborsFinalWeight);
			Filtered.CocRadius = CenterCocRadius;
		}
		#elif AA_TONE || AA_UPSAMPLE
		{
			// Reweight because SampleFinalWeight does not that have total sum = 1.
			Filtered.Color = NeighborsColor * rcp(NeighborsFinalWeight);
			Filtered.CocRadius = 0;
		}
		#else
		{
			Filtered.Color = NeighborsColor;
			Filtered.CocRadius = 0;
		}
		#endif
		
		#if AA_UPSAMPLE
			// Compute the temporal weight of the output pixel.
			IntermediaryResult.FilteredTemporalWeight = ComputePixelWeigth(IntermediaryResult, dKO);
		#endif
	}

	IntermediaryResult.Filtered = Filtered;
}


// Compute the neighborhood bounding box used to reject history.
void ComputeNeighborhoodBoundingbox(
	in FTAAInputParameters InputParams,
	in FTAAIntermediaryResult IntermediaryResult,
	out FTAAHistoryPayload OutNeighborMin,
	out FTAAHistoryPayload OutNeighborMax)
{
	// TODO: clean this up.
	FTAAHistoryPayload Neighbors[kNeighborsCount];
	UNROLL
	for (uint i = 0; i < kNeighborsCount; i++)
	{
		Neighbors[i].Color = SampleCachedSceneColorTexture(InputParams, kOffsets3x3[i]).Color;
		Neighbors[i].CocRadius = SampleCachedSceneColorTexture(InputParams, kOffsets3x3[i]).CocRadius;
	}

	FTAAHistoryPayload NeighborMin;
	FTAAHistoryPayload NeighborMax;

	#if AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_VARIANCE || AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_T_INTERVAL
	{
		#if AA_SAMPLES == 9
			const uint SampleIndexes[9] = kSquareIndexes3x3;
		#elif AA_SAMPLES == 5
			const uint SampleIndexes[5] = kPlusIndexes3x3;
		#else
			#error Unknown number of samples.
		#endif

		FTAAHistoryPayload m1 = (FTAAHistoryPayload)0;
		FTAAHistoryPayload m2 = (FTAAHistoryPayload)0;
		for( uint i = 0; i < AA_SAMPLES; i++ )
		{
			FTAAHistoryPayload Sample = Neighbors[ SampleIndexes[i] ];

			m1 = AddPayload(m1, Sample);
			m2 = AddPayload(m2, Pow2Payload(Sample));
		}

		m1 = MulPayload(m1, (1.0 / AA_SAMPLES));
		m2 = MulPayload(m2, (1.0 / AA_SAMPLES));

		float4 StdDevColor = sqrt( abs(m2.Color - m1.Color * m1.Color) );
		float StdDevCocRadius = sqrt( abs(m2.CocRadius - m1.CocRadius * m1.CocRadius) );
		#if AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_VARIANCE
			NeighborMin.Color = m1.Color - 1.25 * StdDevColor;
			NeighborMin.CocRadius = m1.CocRadius - 1.25 * StdDevCocRadius;
			NeighborMax.Color = m1.Color + 1.25 * StdDevColor;
			NeighborMax.CocRadius = m1.CocRadius + 1.25 * StdDevCocRadius;
		#elif AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_T_INTERVAL
			// We estimate the mean and stddev based on a low sample count
			// so there is some uncertainty in these estimations.
			// We model this uncertainty with a T-distribution,
			// and use this to get a confidence interval.

		#if AA_SAMPLES == 9
			const float StudentTValue = 2.306f; // 95% confidence on 9 samples (8 degrees of freedom)
		#elif AA_SAMPLES == 5
			const float StudentTValue = 2.776f; // 95% confidence on 5 samples (4 degrees of freedom)
		#else
			#error Unknown number of samples.
		#endif
		
			// Prediction Interval Formula
			float4 ErrorMarginColor = StdDevColor * StudentTValue * sqrt(1.0f + (1.0f/AA_SAMPLES));
			float ErrorMarginCocRadius = StdDevCocRadius * StudentTValue * sqrt(1.0f + (1.0f/AA_SAMPLES));

			NeighborMin.Color = m1.Color - ErrorMarginColor;
			NeighborMin.CocRadius = m1.CocRadius - ErrorMarginCocRadius;
			NeighborMax.Color = m1.Color + ErrorMarginColor;
			NeighborMax.CocRadius = m1.CocRadius + ErrorMarginCocRadius;
		#endif

		NeighborMin = MinPayload(NeighborMin, IntermediaryResult.Filtered);
		NeighborMax = MaxPayload(NeighborMax, IntermediaryResult.Filtered);
	}
	#elif AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_SAMPLE_DISTANCE
	// Do color clamping only within a radius.
	{
		float2 PPCo = InputParams.ViewportUV * InputViewSize.xy + TemporalJitterPixels;
		float2 PPCk = floor(PPCo) + 0.5;
		float2 dKO = PPCo - PPCk;
		
		// Sample 4 is is always going to be considered anyway.
		NeighborMin = Neighbors[4];
		NeighborMax = Neighbors[4];
		
		// Reduce distance threshold as upsacale factor increase to reduce ghosting.
		float DistthresholdLerp = UpscaleFactor - 1;
		float DistThreshold = lerp(1.51, 1.3, DistthresholdLerp);

		#if AA_SAMPLES == 9
			const uint Indexes[9] = kSquareIndexes3x3;
		#else
			const uint Indexes[5] = kPlusIndexes3x3;
		#endif

		UNROLL
		for( uint i = 0; i < AA_SAMPLES; i++ )
		{
			uint NeightborId = Indexes[i];
			if (NeightborId != 4)
			{
				float2 dPP = float2(kOffsets3x3[NeightborId]) - dKO;

				FLATTEN
				if (dot(dPP, dPP) < (DistThreshold * DistThreshold))
				{
					NeighborMin = MinPayload(NeighborMin, Neighbors[NeightborId]);
					NeighborMax = MaxPayload(NeighborMax, Neighbors[NeightborId]);
				}
			}
		}
	}
	#elif AA_HISTORY_CLAMPING_BOX == HISTORY_CLAMPING_BOX_MIN_MAX
	{
		NeighborMin = MinPayload3( Neighbors[1], Neighbors[3], Neighbors[4] );
		NeighborMin = MinPayload3( NeighborMin,  Neighbors[5], Neighbors[7] );

		NeighborMax = MaxPayload3( Neighbors[1], Neighbors[3], Neighbors[4] );
		NeighborMax = MaxPayload3( NeighborMax,  Neighbors[5], Neighbors[7] );
		
		#if AA_SAMPLES == 6
		{
			float2 PPCo = InputParams.ViewportUV * InputViewSize.xy + TemporalJitterPixels;
			float2 PPCk = floor(PPCo) + 0.5;
			float2 dKO = PPCo - PPCk;
			
			int2 FifthNeighborOffset = SignFastInt(dKO);

			FTAAHistoryPayload FifthNeighbor;
			FifthNeighbor.Color = SampleCachedSceneColorTexture(InputParams, FifthNeighborOffset).Color;
			FifthNeighbor.CocRadius = SampleCachedSceneColorTexture(InputParams, FifthNeighborOffset).CocRadius;
			
			NeighborMin = MinPayload(NeighborMin, FifthNeighbor);
			NeighborMax = MaxPayload(NeighborMax, FifthNeighbor);
		}
		#elif AA_SAMPLES == 9
		{
			FTAAHistoryPayload NeighborMinPlus = NeighborMin;
			FTAAHistoryPayload NeighborMaxPlus = NeighborMax;

			NeighborMin = MinPayload3( NeighborMin, Neighbors[0], Neighbors[2] );
			NeighborMin = MinPayload3( NeighborMin, Neighbors[6], Neighbors[8] );

			NeighborMax = MaxPayload3( NeighborMax, Neighbors[0], Neighbors[2] );
			NeighborMax = MaxPayload3( NeighborMax, Neighbors[6], Neighbors[8] );

			if( AA_ROUND )
			{
				NeighborMin = AddPayload(MulPayload(NeighborMin, 0.5), MulPayload(NeighborMinPlus, 0.5));
				NeighborMax = AddPayload(MulPayload(NeighborMax, 0.5), MulPayload(NeighborMaxPlus, 0.5));
			}
		}
		#endif
	}
	#else
		#error Unknown history clamping box.
	#endif

	OutNeighborMin = NeighborMin;
	OutNeighborMax = NeighborMax;
}


// Sample history.
FTAAHistoryPayload SampleHistory(in float2 HistoryScreenPosition)
{
	float4 RawHistory0 = 0;
	float4 RawHistory1 = 0;

	// Sample the history using Catmull-Rom to reduce blur on motion.
	#if AA_BICUBIC
	{
		float2 HistoryBufferUV = HistoryScreenPosition * ScreenPosToHistoryBufferUV.xy + ScreenPosToHistoryBufferUV.zw;

		// Clamp HistoryBufferUV so that we don't have to do it entirely for each samples.
		#if AA_MANUALLY_CLAMP_HISTORY_UV
			HistoryBufferUV = clamp(HistoryBufferUV, HistoryBufferUVMinMax.xy, HistoryBufferUVMinMax.zw);
		#endif

		FCatmullRomSamples Samples = GetBicubic2DCatmullRomSamples(HistoryBufferUV, HistoryBufferSize.xy, HistoryBufferSize.zw);
		UNROLL
		for (uint i = 0; i < Samples.Count; i++)
		{
			float2 SampleUV = Samples.UV[i];

			// Clamp SampleUV within HistoryBufferUVMinMax to avoid sampling potential NaN outside view rect.
			// This may look expensive, but Samples.UVDir is actually compile time constant to give a hint on what and how each component can be optimally clamped.
			if (AA_MANUALLY_CLAMP_HISTORY_UV)
			{
				if (Samples.UVDir[i].x < 0)
				{
					SampleUV.x = max(SampleUV.x, HistoryBufferUVMinMax.x);
				}
				else if (Samples.UVDir[i].x > 0)
				{
					SampleUV.x = min(SampleUV.x, HistoryBufferUVMinMax.z);
				}

				if (Samples.UVDir[i].y < 0)
				{
					SampleUV.y = max(SampleUV.y, HistoryBufferUVMinMax.y);
				}
				else if (Samples.UVDir[i].y > 0)
				{
					SampleUV.y = min(SampleUV.y, HistoryBufferUVMinMax.w);
				}
			}

			RawHistory0 += HistoryBuffer_0.SampleLevel(HistoryBufferSampler_0, SampleUV, 0) * Samples.Weight[i];
		}
		RawHistory0 *= Samples.FinalMultiplier;
	}

	// Sample the history using bilinear sampler.
	#else
	{
		// Clamp HistoryScreenPosition to be within viewport.
		if (AA_MANUALLY_CLAMP_HISTORY_UV)
		{
			HistoryScreenPosition = clamp(HistoryScreenPosition, -ScreenPosAbsMax, ScreenPosAbsMax);
		}

		float2 HistoryBufferUV = HistoryScreenPosition * ScreenPosToHistoryBufferUV.xy + ScreenPosToHistoryBufferUV.zw;

		RawHistory0 = HistoryBuffer_0.SampleLevel(HistoryBufferSampler_0, HistoryBufferUV, 0);
	}
	#endif

	#if HISTORY_RENDER_TARGETS == 2
	{
		if (AA_MANUALLY_CLAMP_HISTORY_UV)
		{
			HistoryScreenPosition = clamp(HistoryScreenPosition, -ScreenPosAbsMax, ScreenPosAbsMax);
		}

		float2 HistoryBufferUV = HistoryScreenPosition * ScreenPosToHistoryBufferUV.xy + ScreenPosToHistoryBufferUV.zw;

		RawHistory1 = HistoryBuffer_1.SampleLevel(HistoryBufferSampler_1, HistoryBufferUV, 0);
	}
	#endif
	
	FTAAHistoryPayload HistoryPayload;
	HistoryPayload.Color = RawHistory0;

	#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
		HistoryPayload.CocRadius = RawHistory1.r;
	#else
		HistoryPayload.CocRadius = RawHistory0.a;
	#endif

	#if COMPILER_SUPPORTS_HLSL2021
		CorrectExposure(HistoryPayload.Color.rgb);
	#else
		HistoryPayload.Color.rgb *= HistoryPreExposureCorrection;
	#endif

	HistoryPayload.Color = TransformSceneColor(HistoryPayload.Color);

	return HistoryPayload;
}


// Clamp history.
FTAAHistoryPayload ClampHistory(inout FTAAIntermediaryResult IntermediaryResult, FTAAHistoryPayload History, FTAAHistoryPayload NeighborMin, FTAAHistoryPayload NeighborMax)
{
	#if !AA_CLAMP
		return History;
	
	#elif AA_CLIP
		// Clip history, this uses color AABB intersection for tighter fit.
		//float4 TargetColor = 0.5 * ( NeighborMin + NeighborMax );
		float4 TargetColor = Filtered;

		float ClipBlend = HistoryClip( HistoryColor.rgb, TargetColor.rgb, NeighborMin.rgb, NeighborMax.rgb );
		
		//float DistToClamp = saturate(-ClipBlend) / ( saturate(-ClipBlend) + 1 );
		//float DistToClamp = abs( ClipBlend ) / ( 1 - ClipBlend );
		ClipBlend = saturate( ClipBlend );

		HistoryColor = lerp( HistoryColor, TargetColor, ClipBlend );

		#if AA_FORCE_ALPHA_CLAMP
			HistoryColor.a = clamp( HistoryColor.a, NeighborMin.a, NeighborMax.a );
		#endif

		return HistoryColor;

	#else //!AA_CLIP
		History.Color = clamp(History.Color, NeighborMin.Color, NeighborMax.Color);
		History.CocRadius = clamp(History.CocRadius, NeighborMin.CocRadius, NeighborMax.CocRadius);
		return History;
	#endif
}


//------------------------------------------------------- TAA MAIN FUNCTION

FTAAHistoryPayload TemporalAASample(uint2 GroupId, uint2 GroupThreadId, uint GroupThreadIndex, float2 ViewportUV, float FrameExposureScale)
{
	// SETUP
	// -----
	FTAAInputParameters InputParams;

	// Per frame setup.
	InputParams.FrameExposureScale = ToScalarMemory(FrameExposureScale * View.OneOverPreExposure);


	// Per pixel setup.
	{
		InputParams.GroupId = GroupId;
		InputParams.GroupThreadId = GroupThreadId;
		InputParams.GroupThreadIndex = GroupThreadIndex;
		InputParams.ViewportUV = ViewportUV;
		InputParams.ScreenPos = ViewportUVToScreenPos(ViewportUV);
		InputParams.NearestBufferUV = ViewportUV * ViewportUVToInputBufferUV.xy + ViewportUVToInputBufferUV.zw;

		// Handle single or multi-pass responsive AA
		#if AA_SINGLE_PASS_RESPONSIVE
		{
			const uint kResponsiveStencilMask = 1 << 3;
			
			int2 SceneStencilUV = (int2)trunc(InputParams.NearestBufferUV * InputSceneColorSize.xy);
			uint SceneStencilRef = StencilTexture.Load(int3(SceneStencilUV, 0)) STENCIL_COMPONENT_SWIZZLE;

			InputParams.bIsResponsiveAAPixel = (SceneStencilRef & kResponsiveStencilMask) ? 1.f : 0.f;
		}
		#elif TAA_RESPONSIVE
			InputParams.bIsResponsiveAAPixel = 1.f;
		#else
			InputParams.bIsResponsiveAAPixel = 0.f;
		#endif
	
		#if AA_UPSAMPLE
		{
			// Pixel coordinate of the center of output pixel O in the input viewport.
			float2 PPCo = ViewportUV * InputViewSize.xy + TemporalJitterPixels;
		
			// Pixel coordinate of the center of the nearest input pixel K.
			float2 PPCk = floor(PPCo) + 0.5;
		
			// Pixel coordinate of the center of the nearest top left input pixel T.
			float2 PPCt = floor(PPCo - 0.5) + 0.5;
		
			InputParams.NearestBufferUV = InputSceneColorSize.zw * (InputViewMin + PPCk);
			InputParams.NearestTopLeftBufferUV = InputSceneColorSize.zw * (InputViewMin + PPCt);

			// TODO: because use nearest sampler, can be faster in this computation.
		}
		#endif
	}

	// Setup intermediary results.
	FTAAIntermediaryResult IntermediaryResult = CreateIntermediaryResult();

	// FIND MOTION OF PIXEL AND NEAREST IN NEIGHBORHOOD
	// ------------------------------------------------
	float3 PosN; // Position of this pixel, possibly later nearest pixel in neighborhood.
	PosN.xy = InputParams.ScreenPos;

	PrecacheInputSceneDepth(InputParams);
	PosN.z = SampleCachedSceneDepthTexture(InputParams, int2(0, 0));

	// Screen position of minimum depth.
	float2 VelocityOffset = float2(0.0, 0.0);
	#if AA_CROSS // TODO: 2x2.
	{
		// For motion vector, use camera/dynamic motion from min depth pixel in pattern around pixel.
		// This enables better quality outline on foreground against different motion background.
		// Larger 2 pixel distance "x" works best (because AA dilates surface).
		float4 Depths;
		Depths.x = SampleCachedSceneDepthTexture(InputParams, int2(-AA_CROSS, -AA_CROSS));
		Depths.y = SampleCachedSceneDepthTexture(InputParams, int2( AA_CROSS, -AA_CROSS));
		Depths.z = SampleCachedSceneDepthTexture(InputParams, int2(-AA_CROSS,  AA_CROSS));
		Depths.w = SampleCachedSceneDepthTexture(InputParams, int2( AA_CROSS,  AA_CROSS));

		float2 DepthOffset = float2(AA_CROSS, AA_CROSS);
		float DepthOffsetXx = float(AA_CROSS);
		// Nearest depth is the largest depth (depth surface 0=far, 1=near).
		if(Depths.x > Depths.y) 
		{
			DepthOffsetXx = -AA_CROSS;
		}
		if(Depths.z > Depths.w) 
		{
			DepthOffset.x = -AA_CROSS;
		}
		float DepthsXY = max(Depths.x, Depths.y);
		float DepthsZW = max(Depths.z, Depths.w);
		if(DepthsXY > DepthsZW) 
		{
			DepthOffset.y = -AA_CROSS;
			DepthOffset.x = DepthOffsetXx; 
		}
		float DepthsXYZW = max(DepthsXY, DepthsZW);
		if(DepthsXYZW > PosN.z) 
		{
			// This is offset for reading from velocity texture.
			// This supports half or fractional resolution velocity textures.
			// With the assumption that UV position scales between velocity and color.
			VelocityOffset = DepthOffset * InputSceneColorSize.zw;
			// This is [0 to 1] flipped in Y.
			//PosN.xy = ScreenPos + DepthOffset * OutputViewportSize.zw * 2.0;
			PosN.z = DepthsXYZW;
		}
	}
	#endif	// AA_CROSS

	// Camera motion for pixel or nearest pixel (in ScreenPos space).
	bool OffScreen = false;
	float Velocity = 0;
	float HistoryBlur = 0;
	float2 HistoryScreenPosition = InputParams.ScreenPos;

	#if 1
	{
		float4 ThisClip = float4( PosN.xy, PosN.z, 1 );
		float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
		float2 BackN = PosN.xy - PrevScreen;

		float2 BackTemp = BackN * OutputViewportSize.xy;

		#if AA_DYNAMIC
		{
			ENCODED_VELOCITY_TYPE EncodedVelocity = SampleVelocityTexture(InputParams.NearestBufferUV + VelocityOffset);
			bool DynamicN = EncodedVelocity.x > 0.0;
			if(DynamicN)
			{
				BackN = DecodeVelocityFromTexture(EncodedVelocity).xy;
			}
			BackTemp = BackN * OutputViewportSize.xy;
		}
		#endif

		Velocity = sqrt(dot(BackTemp, BackTemp));
		#if !AA_BICUBIC
			// Save the amount of pixel offset of just camera motion, used later as the amount of blur introduced by history.
			float HistoryBlurAmp = 2.0;
			HistoryBlur = saturate(abs(BackTemp.x) * HistoryBlurAmp + abs(BackTemp.y) * HistoryBlurAmp);
		#endif
		// Easier to do off screen check before conversion.
		// BackN is in units of 2pixels/viewportWidthInPixels
		// This converts back projection vector to [-1 to 1] offset in viewport.
		HistoryScreenPosition = InputParams.ScreenPos - BackN;

		// Detect if HistoryBufferUV would be outside of the viewport.
		OffScreen = max(abs(HistoryScreenPosition.x), abs(HistoryScreenPosition.y)) >= 1.0;
	}
	#endif

	// Precache input scene color.
	PrecacheInputSceneColor(/* inout = */ InputParams);

	// Filter input.
	#if AA_UPSAMPLE_ADAPTIVE_FILTERING == 0
		FilterCurrentFrameInputSamples(
			InputParams,
			/* inout = */ IntermediaryResult);
	#endif
	
	// Compute neighborhood bounding box.
	FTAAHistoryPayload NeighborMin;
	FTAAHistoryPayload NeighborMax;

	ComputeNeighborhoodBoundingbox(
		InputParams,
		/* inout = */ IntermediaryResult,
		NeighborMin, NeighborMax);

	// Sample history.
	FTAAHistoryPayload History = SampleHistory(HistoryScreenPosition);

	// Whether the feedback needs to be reset.
	bool IgnoreHistory = OffScreen || bCameraCut;

	// DYNAMIC ANTI GHOSTING
	// ---------------------
	#if AA_DYNAMIC_ANTIGHOST && AA_DYNAMIC && HISTORY_PAYLOAD_COMPONENTS == 3
	bool Dynamic4;
	{
		#if !AA_DYNAMIC
			#error AA_DYNAMIC_ANTIGHOST requires AA_DYNAMIC
		#endif
		// TODO: try a 2x2 for AA_UPSAMPLE
		bool Dynamic1 = SampleVelocityTexture(InputParams.NearestBufferUV, int2( 0, -1)).x > 0;
		bool Dynamic3 = SampleVelocityTexture(InputParams.NearestBufferUV, int2(-1,  0)).x > 0;
		Dynamic4 = SampleVelocityTexture(InputParams.NearestBufferUV).x > 0;
		bool Dynamic5 = SampleVelocityTexture(InputParams.NearestBufferUV, int2( 1,  0)).x > 0;
		bool Dynamic7 = SampleVelocityTexture(InputParams.NearestBufferUV, int2( 0,  1)).x > 0;

		bool Dynamic = Dynamic1 || Dynamic3 || Dynamic4 || Dynamic5 || Dynamic7;
		IgnoreHistory = IgnoreHistory || (!Dynamic && History.Color.a > 0);
	}
	#endif
	
	// Save off luma of history before the clamp.
	float LumaMin     = GetSceneColorLuma4(NeighborMin.Color);
	float LumaMax     = GetSceneColorLuma4(NeighborMax.Color);
	float LumaHistory = GetSceneColorLuma4(History.Color);
	
	// Clamp history.
	FTAAHistoryPayload PreClampingHistoryColor = History;
	History = ClampHistory(IntermediaryResult, History, NeighborMin, NeighborMax);
	
	// Filter input after color clamping.
	#if AA_UPSAMPLE_ADAPTIVE_FILTERING == 1
	{
		#if AA_VARIANCE
			#error AA_VARIANCE and AA_UPSAMPLE_ADAPTIVE_FILTERING are not compatible because of circular code dependency.
		#endif

		if (IgnoreHistory) // || any(HistoryColor != PreClampingHistoryColor))
		{
			// Set the input filter infinitely large when we know need to rely on it.
			IntermediaryResult.InvFilterScaleFactor = 0;
		}

		// Blur input according to input pixel velocity to reduce blocky filtering cause by narrow filter on low screen percentage.
		// Multiplied by upscale factor because Velocity is in output resolution based pixel velocity.
		IntermediaryResult.InvFilterScaleFactor -= (Velocity * UpscaleFactor) * 0.1;

		// Set a minimal filtering scale to screen percentage to not unecessarily blur the input more than the screen percentage.
		IntermediaryResult.InvFilterScaleFactor = max(IntermediaryResult.InvFilterScaleFactor, ScreenPercentage);

		FilterCurrentFrameInputSamples(
			InputParams,
			/* inout = */ IntermediaryResult);
	}
	#endif

	// ADD BACK IN ALIASING TO SHARPEN
	// -------------------------------
	#if AA_FILTERED && !AA_BICUBIC
	{
		#if AA_UPSAMPLE
			#error Temporal upsample does not support sharpen.
		#endif
		
		// Blend in non-filtered based on the amount of sub-pixel motion.
		float AddAliasing = saturate(HistoryBlur) * 0.5;
		float LumaContrastFactor = 32.0;
		#if AA_YCOCG // TODO: Probably a bug arround here because using Luma4() even with YCOCG=0.
			// 1/4 as bright.
			LumaContrastFactor *= 4.0;
		#endif
		float LumaContrast = LumaMax - LumaMin;
		AddAliasing = saturate(AddAliasing + rcp(1.0 + LumaContrast * LumaContrastFactor));
		IntermediaryResult.Filtered.Color = lerp(IntermediaryResult.Filtered.Color, SampleCachedSceneColorTexture(InputParams, int2(0, 0)).Color, AddAliasing);
	}
	#endif
	
	// COMPUTE BLEND AMOUNT 
	// --------------------
	float BlendFinal;
	{
		float LumaFiltered = GetSceneColorLuma4(IntermediaryResult.Filtered.Color);

		BlendFinal = IntermediaryResult.FilteredTemporalWeight * CurrentFrameWeight;

		BlendFinal = lerp(BlendFinal, 0.2, saturate(Velocity / 40));

		#if 0
		{
			// Anti-flicker
			float DistToClamp = 2 * abs(min(LumaHistory - LumaMin, LumaMax - LumaHistory) / (LumaMax - LumaMin));
			//BlendFinal *= lerp( 0, 1, saturate(4 * DistToClamp) );
			BlendFinal += 0.8 * saturate(0.02 * LumaHistory / abs(Filtered.x - LumaHistory));
			BlendFinal *= (LumaMin * InExposureScale + 0.5) / (LumaMax * InExposureScale + 0.5);
		}
		#endif

		// Make sure to have at least some small contribution
		BlendFinal = max( BlendFinal, saturate( 0.01 * LumaHistory / abs( LumaFiltered - LumaHistory ) ) );

		#if AA_NAN && (COMPILER_GLSL || COMPILER_METAL)
			// The current Metal & GLSL compilers don't handle saturate(NaN) -> 0, instead they return NaN/INF.
			BlendFinal = -min(-BlendFinal, 0.0);
		#endif

		// Responsive forces 1/4 of new frame.
		BlendFinal = InputParams.bIsResponsiveAAPixel ? (1.0/4.0) : BlendFinal;

		#if AA_LERP 
			BlendFinal = 1.0/float(AA_LERP);
		#endif
	
		#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC || AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
		{
			float BilateralWeight = ComputeBilateralWeight(IntermediaryResult.Filtered.CocRadius, History.CocRadius);

			BlendFinal = lerp(BlendFinal, 1, (1-BilateralWeight)*CoCBilateralFilterStrength);
		}
		#endif
		
		if (bCameraCut)
		{
			BlendFinal = 1.0;
		}
	}

	// Offscreen feedback resets.
	if (IgnoreHistory)
	{
		History = IntermediaryResult.Filtered;
		#if HISTORY_PAYLOAD_COMPONENTS == 3
			History.Color.a = 0.0;
		#endif
	}
	
	// DO FINAL BLEND BETWEEN HISTORY AND FILTERED COLOR
	// -------------------------------------------------
	// Luma weighted blend
	float FilterWeight = GetSceneColorHdrWeight(InputParams, IntermediaryResult.Filtered.Color.x);
	float HistoryWeight = GetSceneColorHdrWeight(InputParams, History.Color.x);

	FTAAHistoryPayload OutputPayload;
	{
		float2 Weights = WeightedLerpFactors(HistoryWeight, FilterWeight, BlendFinal);
		OutputPayload = AddPayload(MulPayload(History, Weights.x), MulPayload(IntermediaryResult.Filtered, Weights.y));
	}

	// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
	// (0.995 chosen to accomodate handling of 254/255)
	if (OutputPayload.Color.a > 0.995)
	{
		OutputPayload.Color.a = 1;
	}

	OutputPayload.Color = TransformBackToRawLinearSceneColor(OutputPayload.Color);
	
	#if AA_NAN
		// Transform NaNs to black, transform negative colors to black.
		OutputPayload.Color = -min(-OutputPayload.Color, 0.0);

		OutputPayload.CocRadius = isnan(OutputPayload.CocRadius) ? 0.0 : OutputPayload.CocRadius;
	#endif

	#if HISTORY_PAYLOAD_COMPONENTS == 3
		#if  AA_DYNAMIC_ANTIGHOST && AA_DYNAMIC
			OutputPayload.Color.a = Dynamic4 ? 1 : 0;
		#else
			// Zero out to remove any prior computation of alpha
			OutputPayload.Color.a = 0;
		#endif
	#endif

	return OutputPayload;
}

//------------------------------------------------------- ENTRY POINTS

#if COMPUTESHADER

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float2 ViewportUV = (float2(DispatchThreadId) + 0.5f) * OutputViewportSize.zw;
	
	#if AA_LOWER_RESOLUTION
	{
		ViewportUV = (float2(DispatchThreadId) + 0.5f) * MaxViewportUVAndSvPositionToViewportUV.zw;
		ViewportUV = min(ViewportUV, MaxViewportUVAndSvPositionToViewportUV.xy);
	}
	#endif

	float FrameExposureScale = EyeAdaptationLookup();
	FTAAHistoryPayload OutputPayload = TemporalAASample(GroupId, GroupThreadId, GroupThreadIndex, ViewportUV, FrameExposureScale);

	float4 OutColor0 = 0;
	float4 OutColor1 = 0;

	#if AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_COC
	{
		OutColor0.rgb = OutputPayload.Color.rgb;
		OutColor0.a = OutputPayload.CocRadius;
	}
	#elif AA_HISTORY_PAYLOAD == HISTORY_PAYLOAD_RGB_OPACITY_COC
	{
		OutColor0 = OutputPayload.Color;
		OutColor1.r = OutputPayload.CocRadius;
	}
	#else
	{
		OutColor0 = OutputPayload.Color;
	}
	#endif

	uint2 PixelPos = DispatchThreadId + OutputViewportRect.xy;
	if (all(PixelPos < OutputViewportRect.zw))
	{
		float4 FinalOutput0 = min(MaxHalfFloat.xxxx, OutColor0);
		#if AA_ENABLE_STOCASTIC_QUANTIZATION
		{
			FinalOutput0.rgb = QuantizeFloatColor(FinalOutput0.rgb, OutputQuantizationError, PixelPos, QUANTIZE_NOISE_HAMMERSLEY);
		}
		#endif

		OutComputeTex_0[PixelPos] = FinalOutput0;

		#if HISTORY_RENDER_TARGETS == 2
			OutComputeTex_1[PixelPos] = OutColor1;
		#endif
	}

	#if TAA_DOWNSAMPLE
	{
		// This shader permutation outputs half resolution image in addition to main full-res one.
		// It is more efficient than performing a separate downsampling pass afterwards.
		// Assumes output resolution is always even.

		uint P0 = GroupThreadId.x + GroupThreadId.y * THREADGROUP_SIZEX;
		uint P1 = P0 + 1;
		uint P2 = P0 + THREADGROUP_SIZEX;
		uint P3 = P2 + 1;

		GroupSharedDownsampleArray[P0] = OutColor0;

		GroupMemoryBarrierWithGroupSync();

		if (((GroupThreadId.x | GroupThreadId.y) & 1) == 0)
		{
			OutComputeTexDownsampled[PixelPos / 2] =
				(OutColor0 + GroupSharedDownsampleArray[P1] + GroupSharedDownsampleArray[P2] + GroupSharedDownsampleArray[P3]) * 0.25;
		}
	}
	#endif //TAA_DOWNSAMPLE
}

#elif PIXELSHADER // Mobile Only, supports Main and MainUpsampling

void MainPS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor0 : SV_Target0
	)
{
	float2 ViewportUV = (SvPosition.xy - OutputViewportRect.xy) * OutputViewportSize.zw;

	uint2 GroupId = int2(0, 0);
	uint2 GroupThreadId = int2(0, 0);
	uint GroupThreadIndex = 0;
	float FrameExposureScale = EyeAdaptationLookup();
	
	FTAAHistoryPayload OutputPayload = TemporalAASample(GroupId, GroupThreadId, GroupThreadIndex, ViewportUV, FrameExposureScale);

	OutColor0 = OutputPayload.Color;
}

#endif
