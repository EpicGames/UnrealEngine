// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRClosestOccluder.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#define TILE_ROWS 2


//------------------------------------------------------- PARAMETERS

RWTexture2DArray<uint> PrevAtomicOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void ClearPrevTexturesCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 GroupThreadId = uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);
	uint2 OutputPixelOrigin = (InputInfo_ViewportMin + GroupId * uint(TILE_SIZE * TILE_ROWS)) + GroupThreadId;

	UNROLL_N(TILE_ROWS)
	for (uint y = 0; y < TILE_ROWS; y++)
	{
		UNROLL_N(TILE_ROWS)
		for (uint x = 0; x < TILE_ROWS; x++)
		{
			uint2 PixelOffset = uint2(x, y) * TILE_SIZE;
			uint2 InputPixelPos = OutputPixelOrigin + PixelOffset;

			InputPixelPos.x = select(all(InputPixelPos < InputInfo_ViewportMax), InputPixelPos.x, uint(~0));

			PrevAtomicOutput[uint3(InputPixelPos, 0)] = 0;
			
			#if CONFIG_SUPPORT_ORTHO_VIEW
				PrevAtomicOutput[uint3(InputPixelPos, 1)] = 0;
			#endif
		}
	}
}
