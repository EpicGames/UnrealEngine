// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRCommon.ush"


//------------------------------------------------------- DEFINITIONS

#define CONFIG_SUPPORT_ORTHO_VIEW 1

#define VELOCITY_HOLE_FILLING_BITS 18
#define VELOCITY_HOLE_FILLING_LENGTH_BITS 13
#define VELOCITY_HOLE_FILLING_ANGLE_BITS (VELOCITY_HOLE_FILLING_BITS - VELOCITY_HOLE_FILLING_LENGTH_BITS)
#define VELOCITY_HOLE_FILLING_LENGTH_PRECISION (pow(2.0, 5))


//------------------------------------------------------- PARAMETERS

float2 PrevOutputBufferUVMin;
float2 PrevOutputBufferUVMax;


//------------------------------------------------------- HOLE FILLING VELOCITY ENCODING

CALL_SITE_DEBUGLOC
void PolarCoordinateHoleFillingVelocity(float2 PixelVelocity, out float PixelVelocityAngle, out float PixelVelocityLength)
{
	PixelVelocityAngle = atan2Fast(PixelVelocity.y, PixelVelocity.x);
	PixelVelocityLength = length(PixelVelocity);
}

CALL_SITE_DEBUGLOC
float2 CartesianHoleFillingVelocity(float PixelVelocityAngle, float PixelVelocityLength)
{
	float2 PixelDirection;
	sincos(PixelVelocityAngle, /* out */ PixelDirection.y, /* out */ PixelDirection.x);
	return PixelDirection * PixelVelocityLength;
}

CALL_SITE_DEBUGLOC
void QuantizeHoleFillingVelocity(float2 PixelVelocity, out float EncodedAngle, out float EncodedLength)
{
	const uint AngleBitDepth = 1 << VELOCITY_HOLE_FILLING_ANGLE_BITS;
	const float LengthPixelPrecision = VELOCITY_HOLE_FILLING_LENGTH_PRECISION;

	float PixelVelocityAngle;
	float PixelVelocityLength;
	PolarCoordinateHoleFillingVelocity(PixelVelocity, /* out */ PixelVelocityAngle, /* out */ PixelVelocityLength);

	EncodedAngle = PixelVelocityAngle * (0.5 * AngleBitDepth / PI) + (0.5 * AngleBitDepth);
	EncodedLength = PixelVelocityLength * LengthPixelPrecision;
}

CALL_SITE_DEBUGLOC
uint EncodeHoleFillingVelocity(float2 PixelVelocity)
{
	const uint AngleBitDepth = 1 << VELOCITY_HOLE_FILLING_ANGLE_BITS;
	const uint LengthBitDepth = 1 << VELOCITY_HOLE_FILLING_LENGTH_BITS;

	float EncodedAngle;
	float EncodedLength;
	QuantizeHoleFillingVelocity(PixelVelocity, /* out */ EncodedAngle, /* out */ EncodedLength);

	uint iEncodedAngle = uint(round(EncodedAngle)) & (AngleBitDepth - 1);
	uint iEncodedLength = fastClamp(uint(ceil(EncodedLength)), 0u, LengthBitDepth - 1u);

	uint EncodedHoleFillingVelocity = iEncodedAngle * LengthBitDepth + iEncodedLength;
	return EncodedHoleFillingVelocity;
}

CALL_SITE_DEBUGLOC
void DecodeHoleFillingVelocity(uint EncodedHoleFillingVelocity, out float PixelAngle, out float PixelLength)
{
	const uint AngleBitDepth = 1 << VELOCITY_HOLE_FILLING_ANGLE_BITS;
	const uint LengthBitDepth = 1 << VELOCITY_HOLE_FILLING_LENGTH_BITS;
	const float LengthPixelPrecision = VELOCITY_HOLE_FILLING_LENGTH_PRECISION;

	uint iEncodedLength = EncodedHoleFillingVelocity % LengthBitDepth;
	uint iEncodedAngle = (EncodedHoleFillingVelocity >> VELOCITY_HOLE_FILLING_LENGTH_BITS) % AngleBitDepth;

	float EncodedLength = float(iEncodedLength);
	PixelLength = EncodedLength * rcp(LengthPixelPrecision);

	float EncodedAngle = float(iEncodedAngle);
	PixelAngle = EncodedAngle * (PI * 2.0 / float(AngleBitDepth)) - PI;
}

CALL_SITE_DEBUGLOC
float GetMaxEncodableHoleFillingPixelLength()
{
	const uint LengthBitDepth = 1 << VELOCITY_HOLE_FILLING_LENGTH_BITS;
	const float LengthPixelPrecision = VELOCITY_HOLE_FILLING_LENGTH_PRECISION;
	const float LengthRange = (float(LengthBitDepth) - 1.5) / LengthPixelPrecision;
	return LengthRange;
}


//------------------------------------------------------- CLOSEST OCCLUDER SCATTERING

void ScatterClosestOccluder(globallycoherent RWTexture2DArray<uint> PrevAtomicOutput, bool bValidOutputPixel, float2 ScreenPos, float2 FinalScreenVelocity, float PrevClosestDeviceZ)
{
	const float PrevUseCountQuantization = 63.0;

	float2 PrevScreenPos = ScreenPos - FinalScreenVelocity.xy;
	float2 PrevInputBufferUV = (InputInfo_ScreenPosToViewportScale * PrevScreenPos + InputInfo_ScreenPosToViewportBias) * InputInfo_ExtentInverse;

	uint EncodedHoleFillingVelocity = EncodeHoleFillingVelocity(FinalScreenVelocity.xy * ScreenVelocityToInputPixelVelocity);

	bool bValidHistoryCoord = bValidOutputPixel && all(and(PrevInputBufferUV > PrevOutputBufferUVMin, PrevInputBufferUV < PrevOutputBufferUVMax));

	FBilinearSampleInfos BilinearInter = GetBilinearSampleLevelInfos(PrevInputBufferUV, InputInfo_Extent, InputInfo_ExtentInverse);
	
	#if CONFIG_SUPPORT_ORTHO_VIEW
	BRANCH
	if (IsOrthoProjection())
	{
		uint Depth = clamp(uint(round(PrevClosestDeviceZ * 8388607.0)), 0, 8388607) << 9;

		uint PrevClosestDepth0 = Depth | ((EncodedHoleFillingVelocity >> 9u) & 0x1FFu);
		uint PrevClosestDepth1 = Depth | ((EncodedHoleFillingVelocity >> 0u) & 0x1FFu);
		
		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			float Weight = GetSampleWeight(BilinearInter, i);
			uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);

			uint FinalAdd = uint(round(Weight * PrevUseCountQuantization));
			
			PixelPos.x = ((FinalAdd > 0) && bValidHistoryCoord) ? PixelPos.x : (~0);

			uint PreviousDepth0;
			InterlockedMax(PrevAtomicOutput[tsr_ushort3(PixelPos, 0)], PrevClosestDepth0, /* out */ PreviousDepth0);

			PixelPos.x = select(PreviousDepth0 < PrevClosestDepth0, PixelPos.x, ~0u);
			InterlockedMax(PrevAtomicOutput[tsr_ushort3(PixelPos, 1)], PrevClosestDepth1);
		}
	}
	else
	#endif // CONFIG_SUPPORT_ORTHO_VIEW
	{
		uint PrevClosestDepth = (f32tof16(PrevClosestDeviceZ) << VELOCITY_HOLE_FILLING_BITS) | EncodedHoleFillingVelocity;
			
		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			float Weight = GetSampleWeight(BilinearInter, i);
			uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);

			uint FinalAdd = uint(round(Weight * PrevUseCountQuantization));
			
			PixelPos.x = ((FinalAdd > 0) && bValidHistoryCoord) ? PixelPos.x : (~0);

			InterlockedMax(PrevAtomicOutput[tsr_ushort3(PixelPos, 0)], PrevClosestDepth);
		}
	}
} // ScatterClosestOccluder()

void LoadPrevAtomicTexturesSamples(
	Texture2DArray<uint> PrevAtomicTextureArray,
	float2 PrevScreenPos,
	out uint HistoryClosestDeviceZSamples0[4],
	out uint HistoryClosestDeviceZSamples1[4])
{
	float2 PrevInputBufferUV = (InputInfo_ScreenPosToViewportScale * PrevScreenPos + InputInfo_ScreenPosToViewportBias) * InputInfo_ExtentInverse;
	PrevInputBufferUV = INVARIANT(PrevInputBufferUV);
	FBilinearSampleInfos BilinearInter = GetBilinearSampleLevelInfos(PrevInputBufferUV, InputInfo_Extent, InputInfo_ExtentInverse);

	UNROLL_N(4)
	for (uint i = 0; i < 4; i++)
	{
		float BilinearWeight = GetSampleWeight(BilinearInter, i);
		uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);

		HistoryClosestDeviceZSamples0[i] = PrevAtomicTextureArray[uint3(PixelPos, 0)];

		#if CONFIG_SUPPORT_ORTHO_VIEW
		{
			HistoryClosestDeviceZSamples1[i] = PrevAtomicTextureArray[uint3(PixelPos, 1)];
		}
		#else
		{
			HistoryClosestDeviceZSamples1[i] = 0;
		}
		#endif
	}
} // LoadPrevAtomicTexturesSamples()

void ProcessPrevAtomicTexturesSamples(
	uint HistoryClosestDeviceZSamples0[4],
	uint HistoryClosestDeviceZSamples1[4],
	float2 ScreenPos,
	float2 ScreenVelocity,
	float PrevDeviceZ,
	float DeviceZError,
	bool bIsOffScreen,
	out bool bIsParallaxDisocclusion,
	out float2 HoleFillingPixelVelocity,
	out bool bCanHoleFill)
{
	float WorldDepth = ConvertFromDeviceZ(PrevDeviceZ);
	float WorldDepthError = abs(WorldDepth - ConvertFromDeviceZ(PrevDeviceZ + DeviceZError));
	
	float2 PrevScreenPos = ScreenPos - ScreenVelocity;
	float2 PrevInputBufferUV = (InputInfo_ScreenPosToViewportScale * PrevScreenPos + InputInfo_ScreenPosToViewportBias) * InputInfo_ExtentInverse;
	PrevInputBufferUV = INVARIANT(PrevInputBufferUV);
	FBilinearSampleInfos BilinearInter = GetBilinearSampleLevelInfos(PrevInputBufferUV, InputInfo_Extent, InputInfo_ExtentInverse);
	
	tsr_half ParallaxRejectionMask = tsr_half(0.0);
	bool bIsValidHoleFillingPixelVelocity = false;
	uint EncodedHoleFillingVelocity = 0;
	#if CONFIG_SUPPORT_ORTHO_VIEW
	BRANCH
	if (IsOrthoProjection())
	{
		float ClosestDeviceZ = 0.0f;

		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			float BilinearWeight = GetSampleWeight(BilinearInter, i);
			uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);

			bool bValidPixelPos = all(and(PixelPos >= InputInfo_ViewportMin, PixelPos < InputInfo_ViewportMax));
			
			uint SampleHistoryClosestDeviceZ0 = HistoryClosestDeviceZSamples0[i];
			uint SampleHistoryClosestDeviceZ1 = HistoryClosestDeviceZSamples1[i];
		
			uint SampleEncodedHoleFillingVelocity = ((SampleHistoryClosestDeviceZ0 & 0x1FFu) << 9) | (SampleHistoryClosestDeviceZ1 & 0x1FFu);

			float HistoryClosestDeviceZ = float((SampleHistoryClosestDeviceZ0 & 0xFFFFFE00u) >> 9u) * rcp(8388607.0);
			float HistoryClosestWorldDepth = ConvertFromDeviceZ(HistoryClosestDeviceZ);

			tsr_half DepthRejection;
			{
				const float PixelDepthError = 3.0;
				const float RadiusToDiameter = 2.0;
				float WorldDepthEpsilon = GetDepthPixelRadiusForProjectionType(HistoryClosestWorldDepth) * PixelDepthError * RadiusToDiameter;
				WorldDepthEpsilon += WorldDepthError;
				float DeltaDepth = abs(HistoryClosestWorldDepth - WorldDepth);

				DepthRejection = saturate(tsr_half(2.0) - tsr_half(DeltaDepth / WorldDepthEpsilon));
			}

			bIsValidHoleFillingPixelVelocity = bIsValidHoleFillingPixelVelocity || bValidPixelPos;

			FLATTEN
			if (bValidPixelPos && HistoryClosestDeviceZ > ClosestDeviceZ)
			{
				EncodedHoleFillingVelocity = SampleEncodedHoleFillingVelocity;
			}

			ParallaxRejectionMask = select(bValidPixelPos, ParallaxRejectionMask + tsr_half(BilinearWeight) * DepthRejection, ParallaxRejectionMask);
		}
	}
	else
	#endif // CONFIG_SUPPORT_ORTHO_VIEW
	{
		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
		{
			float BilinearWeight = GetSampleWeight(BilinearInter, i);
			uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);

			bool bValidPixelPos = all(and(PixelPos >= InputInfo_ViewportMin, PixelPos < InputInfo_ViewportMax));
			
			uint SampleHistoryClosestDeviceZ = HistoryClosestDeviceZSamples0[i];
		
			float HistoryClosestDeviceZ = f16tof32(SampleHistoryClosestDeviceZ >> VELOCITY_HOLE_FILLING_BITS);
			float HistoryClosestWorldDepth = ConvertFromDeviceZ(HistoryClosestDeviceZ);

			tsr_half DepthRejection;
			{
				const float PixelDepthError = 3.0;
				const float RadiusToDiameter = 2.0;
				float WorldDepthEpsilon = GetDepthPixelRadiusForProjectionType(HistoryClosestWorldDepth) * PixelDepthError * RadiusToDiameter;
				WorldDepthEpsilon += WorldDepthError;
				float DeltaDepth = abs(HistoryClosestWorldDepth - WorldDepth);

				DepthRejection = saturate(tsr_half(2.0) - tsr_half(DeltaDepth / WorldDepthEpsilon));
			}

			bIsValidHoleFillingPixelVelocity = bIsValidHoleFillingPixelVelocity || bValidPixelPos;

			EncodedHoleFillingVelocity = select(bValidPixelPos, max(EncodedHoleFillingVelocity, SampleHistoryClosestDeviceZ), EncodedHoleFillingVelocity);
			ParallaxRejectionMask = select(bValidPixelPos, ParallaxRejectionMask + tsr_half(BilinearWeight) * DepthRejection, ParallaxRejectionMask);
		}
	}
	
	float HoleFillingPixelAngle;
	float HoleFillingPixelLength;
	DecodeHoleFillingVelocity(EncodedHoleFillingVelocity, /* out */ HoleFillingPixelAngle, /* out */ HoleFillingPixelLength);

	bool bIsEncodablePixelLength = HoleFillingPixelLength < GetMaxEncodableHoleFillingPixelLength();

	HoleFillingPixelVelocity = CartesianHoleFillingVelocity(HoleFillingPixelAngle, HoleFillingPixelLength);
	bCanHoleFill = bIsValidHoleFillingPixelVelocity && bIsEncodablePixelLength;

	// Dim down the parallax rejection mask if the hole fill velocity matches
	FLATTEN
	if (bIsEncodablePixelLength)
	{
		const float AnglePrecision = 2.0 * PI * pow(0.5, VELOCITY_HOLE_FILLING_ANGLE_BITS);
		const float LengthPixelPrecision = VELOCITY_HOLE_FILLING_LENGTH_PRECISION;

		float2 PixelVelocity = ScreenVelocity * ScreenVelocityToInputPixelVelocity;

		float PixelVelocityAngle, PixelVelocityLength;
		PolarCoordinateHoleFillingVelocity(PixelVelocity, /* out */ PixelVelocityAngle, /* out */ PixelVelocityLength);
		
		float CartesianCoordinatePrecision = AnglePrecision * PixelVelocityLength;

		float PixelLengthDiff = abs(HoleFillingPixelLength - PixelVelocityLength) - 2.0;
		float PixelAngleDiff = abs(PixelVelocityAngle - HoleFillingPixelAngle);
		PixelAngleDiff = min(PixelAngleDiff, 2.0 * PI - PixelAngleDiff);
		
		float VelocityDiff = length(HoleFillingPixelVelocity - PixelVelocity);
		float PixelAngleCompare = saturate(2.0 - PixelAngleDiff / AnglePrecision);
		float PixelLengthCompare = saturate(1.0 + rcp(LengthPixelPrecision) - PixelLengthDiff);

		float MinParallaxRejectionMaskPolar = min(PixelLengthCompare, PixelAngleCompare);
		float MinParallaxRejectionMaskCartesian = saturate(1.0 + CartesianCoordinatePrecision - VelocityDiff);
		
		float MinParallaxRejectionMask = lerp(MinParallaxRejectionMaskCartesian, MinParallaxRejectionMaskPolar, saturate(min(HoleFillingPixelLength, PixelVelocityLength) - 2.0));

		ParallaxRejectionMask = max(ParallaxRejectionMask, tsr_half(MinParallaxRejectionMask));
	}

	bIsParallaxDisocclusion = !bIsOffScreen && ParallaxRejectionMask < tsr_half(0.5);
} // ProcessPrevAtomicTexturesSamples()
