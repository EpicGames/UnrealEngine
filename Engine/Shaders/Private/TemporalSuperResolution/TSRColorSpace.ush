// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRCommon.ush"


//------------------------------------------------------- DEFINITION

/**
 * SMCS = Shading Measurement Color Space:
 *  Goal of this color space is to match tone mapping curve as much as possible.
 *  Note we want the shading measurement in a linear color as oposed to gama encoded color, to not increase the rejection measurements in the shadows.
 *
 *  UE's tonemaping curve is quite a bit complicated, so here is an aproximation:
 *
 *  // https://knarkowicz.wordpress.com/page/2/
 *  float3 ACESFilm(float3 x)
 *  {
 *      float a = 2.51f;
 *      float b = 0.03f;
 *      float c = 2.43f;
 *      float d = 0.59f;
 *      float e = 0.14f;
 *      return saturate((x * (a * x + b)) / (x * (c * x + d) + e));
 *  }
 *
 *  One thing thaw is since TSR is running before bloom, wants a bit more precision in the highlight than ACES curve for bloom stability.
 *  So the Linear -> SMCS curve chosen is SMCS(Linear) -> (Linear / (Linear + 0.17))^2
 *
 * GSC = Guide Color Space:
 *  Color space used to encode SMCS in the TSR.History.Guide. Similarily to the back buffer being gama encoded, GSC is also gamma encoded to keep some precision in the shadows.
 *  It also ensures the error introduced by the QuantizeForUNormRenderTarget() distributed through the gama curve as tonemapper does in the backbuffer.
 *
 *  For performance motivation of the GCSToSMCS() used in the RejectShading pass, we take SMCS(GSC) = GSC^2, leading to GSC(Linear) -> sqrt(SMCS(Linear)) -> Linear / (Linear + 0.17)
 */


//------------------------------------------------------- CONSTANTS

// Largest encodable scene color value to not overflaw in Luma4() with halfs on console
static const tsr_half3 LargestSceneColorRGB  = (LargestNormalNumber * tsr_half(0.25)).xxx;
static const tsr_half4 LargestSceneColorRGBA = (LargestNormalNumber * tsr_half(0.25)).xxxx;

// Weight of each channel for the computation of luminance.
static const tsr_half kMoireLumaToChannel = tsr_half(3.0);
static const tsr_half3 kMoireSMCSWeights = rcp(kMoireLumaToChannel).xxx;

static const float kHistoryAccumulationPreceptionAdd = 1.0;
static const float kGCSPreceptionAdd = 0.17;

// Some bright pixel can cause HdrWeight to get nullified under fp16 representation. So clamping this to a value close to the minimum float float positive value (0.000061).
#define HDR_WEIGHT_SAFE_MIN_VALUE 0.0001


//------------------------------------------------------- LINEAR <-> GUIDE COLOR SPACE

CALL_SITE_DEBUGLOC
tsr_half3 LinearToGCS(tsr_half3 LinearColor)
{
	tsr_half3 GColor = LinearColor  * rcp(LinearColor + tsr_half(kGCSPreceptionAdd));
	return GColor;
}

CALL_SITE_DEBUGLOC
tsr_half4 LinearToGCS(tsr_half4 LinearColor)
{
	return tsr_half4(LinearToGCS(LinearColor.rgb), LinearColor.a);
}

CALL_SITE_DEBUGLOC
tsr_half3 GCSToLinear(tsr_half3 GColor)
{
	tsr_half3 Minus = tsr_half(1.0) - GColor;
	tsr_half3 LDRToLinear = min(tsr_half(kGCSPreceptionAdd) * rcp(Minus), tsr_half(MaxHalfFloat));
	tsr_half3 LinearColor = GColor * LDRToLinear;
	return LinearColor;
}

CALL_SITE_DEBUGLOC
tsr_half4 GCSToLinear(tsr_half4 GColor)
{
	return tsr_half4(GCSToLinear(GColor.rgb), GColor.a);
}

CALL_SITE_DEBUGLOC
tsr_half3 PreExposureCorrectGCS(tsr_half3 GColor, const float PreExposureCorrection)
{
	return LinearToGCS(GCSToLinear(GColor) * tsr_half(PreExposureCorrection));
}

CALL_SITE_DEBUGLOC
tsr_half4 PreExposureCorrectGCS(tsr_half4 GColor, const float PreExposureCorrection)
{
	return tsr_half4(PreExposureCorrectGCS(GColor.rgb, PreExposureCorrection), GColor.a);
}


//------------------------------------------------------- GUIDE COLOR SPACE <-> SHADING MEASUREMENT COLOR SPACE

CALL_SITE_DEBUGLOC
tsr_half GCSToSMCS(tsr_half GColor)
{
	tsr_half SMColor = GColor * GColor;
	return SMColor;
}

CALL_SITE_DEBUGLOC
tsr_half3 GCSToSMCS(tsr_half3 GColor)
{
	tsr_half3 SMColor = GColor * GColor;
	return SMColor;
}

CALL_SITE_DEBUGLOC
tsr_half4 GCSToSMCS(tsr_half4 GColor)
{
	return tsr_half4(GCSToSMCS(GColor.rgb), GColor.a);
}

CALL_SITE_DEBUGLOC
tsr_half3 SMCSToGCS(tsr_half3 SMColor)
{
	tsr_half3 GColor = sqrt(SMColor);
	return GColor;
}

CALL_SITE_DEBUGLOC
tsr_half4 SMCSToGCS(tsr_half4 GColor)
{
	return tsr_half4(SMCSToGCS(GColor.rgb), GColor.a);
}


//------------------------------------------------------- LINEAR <-> SHADING MEASUREMENT COLOR SPACE

CALL_SITE_DEBUGLOC
tsr_half3 LinearToSMCS(tsr_half3 LinearColor)
{
	return GCSToSMCS(LinearToGCS(LinearColor));
}

CALL_SITE_DEBUGLOC
tsr_half4 LinearToSMCS(tsr_half4 LinearColor)
{
	return tsr_half4(LinearToSMCS(LinearColor.rgb), LinearColor.a);
}

CALL_SITE_DEBUGLOC
tsr_half3 SMCSToLinear(tsr_half3 SMColor)
{
	return GCSToLinear(SMCSToGCS(SMColor));
}

CALL_SITE_DEBUGLOC
tsr_half4 SMCSToLinear(tsr_half4 SMColor)
{
	return tsr_half4(SMCSToLinear(SMColor.rgb), SMColor.a);
}


//------------------------------------------------------- KARIS HDR WEIGHTING

// Faster but less accurate luma computation. 
// Luma includes a scaling by 4.
CALL_SITE_DEBUGLOC
tsr_half2 Luma4(tsr_half3x2 Color)
{
	return (Color[1] * tsr_half(2.0)) + (Color[0] + Color[2]);
}

CALL_SITE_DEBUGLOC
tsr_half2 Luma4(tsr_half4x2 Color)
{
	return Luma4(tsr_half3x2(Color[0], Color[1], Color[2]));
}

CALL_SITE_DEBUGLOC
tsr_half Luma4(tsr_half3 Color)
{
	return dpv_lo(Luma4(dpv_interleave_mono_registers(Color)));
}

CALL_SITE_DEBUGLOC
tsr_half Luma4(tsr_half4 Color)
{
	return Luma4(Color.rgb);
}

CALL_SITE_DEBUGLOC
tsr_half2 HdrWeightY(tsr_half2 Luma)
{
	tsr_half Exposure = tsr_half(1.0);

	return max(tsr_half(HDR_WEIGHT_SAFE_MIN_VALUE), rcp(Luma * Exposure + tsr_half(4.0)));
}

CALL_SITE_DEBUGLOC
tsr_half HdrWeightY(tsr_half Luma)
{
	return dpv_lo(HdrWeightY(dpv_interleave_mono_registers(Luma)));
}

CALL_SITE_DEBUGLOC
tsr_half2 HdrWeightInvY(tsr_half2 LDRLuma) 
{
	return tsr_half(4.0) * rcp(tsr_half(1.0) - LDRLuma);
}

CALL_SITE_DEBUGLOC
tsr_half HdrWeightInvY(tsr_half LDRLuma) 
{
	return dpv_lo(HdrWeightInvY(dpv_interleave_mono_registers(LDRLuma)));
}

CALL_SITE_DEBUGLOC
tsr_half2 HdrWeight4(tsr_half3x2 Color)
{
	return HdrWeightY(Luma4(Color));
}

CALL_SITE_DEBUGLOC
tsr_half2 HdrWeight4(tsr_half4x2 Color)
{
	return HdrWeightY(Luma4(Color));
}

CALL_SITE_DEBUGLOC
tsr_half HdrWeight4(tsr_half3 Color)
{
	return HdrWeightY(Luma4(Color));
}

CALL_SITE_DEBUGLOC
tsr_half HdrWeight4(tsr_half4 Color)
{
	return HdrWeightY(Luma4(Color));
}


//------------------------------------------------------- BACK BUFFER ERROR

CALL_SITE_DEBUGLOC
tsr_half MeasureBackbufferLDRQuantizationError()
{
	// Assume the backbuffer is 10bit per channels
	return tsr_half(0.5 / 1024.0);
}


//------------------------------------------------------- STOCASTIC QUANTIZATION

CALL_SITE_DEBUGLOC
tsr_half3x2 QuantizeForUNormRenderTarget(tsr_half3x2 Color, tsr_half E, const float3 QuantizationError)
{
	return Color + dpv_interleave_mono_registers(tsr_half3(QuantizationError)) * (E - tsr_half(0.5));
}

CALL_SITE_DEBUGLOC
tsr_half3 QuantizeForUNormRenderTarget(tsr_half3 Color, tsr_half E, const float3 QuantizationError)
{
	return dpv_lo(QuantizeForUNormRenderTarget(dpv_interleave_mono_registers(Color), E, QuantizationError));
}

CALL_SITE_DEBUGLOC
tsr_half3x2 QuantizeForFloatRenderTarget(tsr_half3x2 Color, tsr_half E, const float3 QuantizationError)
{
	tsr_half3x2 Error = dpv_mul(Color, tsr_half3(QuantizationError));

	#if 0
		// NOP
	#elif CONFIG_COMPILE_FP16
	{
		Error[0] = asfloat16(asint16(Error[0]) & uint16_t(~0x03FF));
		Error[1] = asfloat16(asint16(Error[1]) & uint16_t(~0x03FF));
		Error[2] = asfloat16(asint16(Error[2]) & uint16_t(~0x03FF));
	}
	#else
	{
		Error[0] = asfloat(asuint(Error[0]) & ~0x007FFFFF);
		Error[1] = asfloat(asuint(Error[1]) & ~0x007FFFFF);
		Error[2] = asfloat(asuint(Error[2]) & ~0x007FFFFF);
	}
	#endif
	
	return Color + Error * E;
}

CALL_SITE_DEBUGLOC
tsr_half3 QuantizeForFloatRenderTarget(tsr_half3 Color, tsr_half E, const float3 QuantizationError)
{
	return dpv_lo(QuantizeForFloatRenderTarget(dpv_interleave_mono_registers(Color), E, QuantizationError));
}
