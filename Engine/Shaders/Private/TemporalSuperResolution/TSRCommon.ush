// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "../Random.ush"
#include "../TextureSampling.ush"
#include "../FastMath.ush"
#include "../MonteCarlo.ush"
#include "../ScreenPass.ush"
#include "/Engine/Public/DualPixelVectorization.ush"
#include "/Engine/Public/WaveBroadcastIntrinsics.ush"


//------------------------------------------------------- RECOMPILE HASH

#pragma message("UESHADERMETADATA_VERSION F3C5FF82-E809-4903-98CD-5DC7E2254C87")


//------------------------------------------------------- COMPILER CONFIG

// Generate vector truncation warnings to errors.
#pragma warning(error: 3206)


//------------------------------------------------------- CONFIG

#define DEBUG_OUTPUT 0

#if defined(DIM_16BIT_VALU)
	#define CONFIG_COMPILE_FP16 DIM_16BIT_VALU
#elif PLATFORM_SUPPORTS_REAL_TYPES
	#define CONFIG_COMPILE_FP16 1
#else
	#define CONFIG_COMPILE_FP16 0
#endif

#define CONFIG_SCENE_COLOR_OVERFLOW 1

#if defined(DIM_ALPHA_CHANNEL)
	#define CONFIG_SCENE_COLOR_ALPHA DIM_ALPHA_CHANNEL
#else
	#define CONFIG_SCENE_COLOR_ALPHA 0
#endif

#define CONFIG_ENABLE_STOCASTIC_QUANTIZATION (!CONFIG_SCENE_COLOR_ALPHA)

// DXC allows changing order between multiply and adds https://github.com/microsoft/DirectXShaderCompiler/blob/main/docs/DXIL.rst#precise-qualifier
// which on 16bit, if linear color spaces ends up close to half(65504.0), this can go +inf on if the compiler transform a * 0.5 + b * 0.5 into (a + b) * 0.5.
#define CONFIG_FP16_PRECISE_MULTIPLY_ORDER 1


//------------------------------------------------------- CONSTANTS

/* Maximum number of sample. */
#define MAX_SAMPLE_COUNT 8

#if CONFIG_COMPILE_FP16
	#define tsr_half  half
	#define tsr_half2 half2
	#define tsr_half3 half3
	#define tsr_half4 half4

	#define tsr_short  int16_t
	#define tsr_short2 int16_t2
	#define tsr_short3 int16_t3
	#define tsr_short4 int16_t4

	#define tsr_ushort  uint16_t
	#define tsr_ushort2 uint16_t2
	#define tsr_ushort3 uint16_t3
	#define tsr_ushort4 uint16_t4

	#define tsr_half2x2 half2x2
	#define tsr_half3x2 half3x2
	#define tsr_half4x2 half4x2

	#define tsr_short2x2 int16_t2x2
	#define tsr_short3x2 int16_t3x2
	#define tsr_short4x2 int16_t4x2

	#define tsr_ushort2x2 uint16_t2x2
	#define tsr_ushort3x2 uint16_t3x2
	#define tsr_ushort4x2 uint16_t4x2
	
	#define as_tsr_short(x)  asint16(x)
	#define as_tsr_ushort(x) asuint16(x)
	#define as_tsr_half(x)   asfloat16(x)

#else
	#define tsr_half  float
	#define tsr_half2 float2
	#define tsr_half3 float3
	#define tsr_half4 float4

	#define tsr_short  int
	#define tsr_short2 int2
	#define tsr_short3 int3
	#define tsr_short4 int4

	#define tsr_ushort  uint
	#define tsr_ushort2 uint2
	#define tsr_ushort3 uint3
	#define tsr_ushort4 uint4

	#define tsr_half2x2 float2x2
	#define tsr_half3x2 float3x2
	#define tsr_half4x2 float4x2

	#define tsr_short2x2 int2x2
	#define tsr_short3x2 int3x2
	#define tsr_short4x2 int4x2

	#define tsr_ushort2x2 uint2x2
	#define tsr_ushort3x2 uint3x2
	#define tsr_ushort4x2 uint4x2
	
	#define as_tsr_short(x)  asint(x)
	#define as_tsr_ushort(x) asuint(x)
	#define as_tsr_half(x)   asfloat(x)

#endif

#if CONFIG_SCENE_COLOR_ALPHA
	#define tsr_halfC tsr_half4
	#define tsr_halfCx2 tsr_half4x2
	#define CONFIG_CHANNEL_COUNT 4

#else
	#define tsr_halfC tsr_half3
	#define tsr_halfCx2 tsr_half3x2
	#define CONFIG_CHANNEL_COUNT 3

#endif

// On platforms that compile via SPIRV (PLATFORM_USES_SPIRV_RESOURCE_TYPES), Vulkan SPIRV requires 32-bit sampled types for OpTypeImage
#if CONFIG_COMPILE_FP16 && PLATFORM_USES_SPIRV_RESOURCE_TYPES
	#define tsr_half_resource   float
	#define tsr_half2_resource  float2
	#define tsr_half3_resource  float3
	#define tsr_half4_resource  float4
	#define tsr_ushort_resource uint
#else
	#define tsr_half_resource   tsr_half
	#define tsr_half2_resource  tsr_half2
	#define tsr_half3_resource  tsr_half3
	#define tsr_half4_resource  tsr_half4
	#define tsr_ushort_resource tsr_ushort
#endif

#if CONFIG_SCENE_COLOR_ALPHA
	#define tsr_halfC_resource tsr_half4_resource
#else
	#define tsr_halfC_resource tsr_half3_resource
#endif


// Largest encodable normal number in a half used on console.
static const tsr_half LargestNormalNumber = tsr_half(65504.0);


//------------------------------------------------------- PARAMETERS


float2 InputInfo_Extent;
float2 InputInfo_ExtentInverse;
float2 InputInfo_ScreenPosToViewportScale;
float2 InputInfo_ScreenPosToViewportBias;
uint2  InputInfo_ViewportMin;
uint2  InputInfo_ViewportMax;
float2 InputInfo_ViewportSize;
float2 InputInfo_ViewportSizeInverse;
float2 InputInfo_UVViewportMin;
float2 InputInfo_UVViewportMax;
float2 InputInfo_UVViewportSize;
float2 InputInfo_UVViewportSizeInverse;
float2 InputInfo_UVViewportBilinearMin;
float2 InputInfo_UVViewportBilinearMax;
float2 InputJitter;
int2   InputPixelPosMin;
int2   InputPixelPosMax;
FScreenTransform InputPixelPosToScreenPos;
float2 ScreenVelocityToInputPixelVelocity;
float2 InputPixelVelocityToScreenVelocity;

float2 HistoryInfo_Extent;
float2 HistoryInfo_ExtentInverse;
uint2  HistoryInfo_ViewportMin;
uint2  HistoryInfo_ViewportMax;
float2 HistoryInfo_ViewportSize;
float2 HistoryInfo_ViewportSizeInverse;
float2 HistoryInfo_UVViewportBilinearMin;
float2 HistoryInfo_UVViewportBilinearMax;

// FTSRPrevHistoryParameters
float2 PrevHistoryInfo_Extent;
float2 PrevHistoryInfo_ExtentInverse;
float2 PrevHistoryInfo_ScreenPosToViewportScale;
float2 PrevHistoryInfo_ScreenPosToViewportBias;
uint2  PrevHistoryInfo_ViewportMin;
uint2  PrevHistoryInfo_ViewportMax;
float2 PrevHistoryInfo_ViewportSize;
float2 PrevHistoryInfo_ViewportSizeInverse;
float2 PrevHistoryInfo_UVViewportMin;
float2 PrevHistoryInfo_UVViewportMax;
float2 PrevHistoryInfo_UVViewportSize;
float2 PrevHistoryInfo_UVViewportSizeInverse;
float2 PrevHistoryInfo_UVViewportBilinearMin;
float2 PrevHistoryInfo_UVViewportBilinearMax;
FScreenTransform ScreenPosToPrevHistoryBufferUV;
float HistoryPreExposureCorrection;
float ResurrectionPreExposureCorrection;

uint bCameraCut;

#if DEBUG_OUTPUT
	RWTexture2DArray<float4> DebugOutput;
#endif


//------------------------------------------------------- UTILITY
	
uint sign_bit(float x)
{
    return asuint(x) >> 31u;
}

uint2 sign_bit(float2 x)
{
    return uint2(sign_bit(x.x), sign_bit(x.y));
}

#if PLATFORM_SUPPORTS_REAL_TYPES

uint16_t sign_bit(half x)
{
    return asuint16(x) >> uint16_t(15);
}

uint16_t2 sign_bit(half2 x)
{
    return asuint16(x) >> uint16_t(15);
}

#endif


//------------------------------------------------------- FUNCTIONS
	
CALL_SITE_DEBUGLOC 
float2x2 ApplyScreenTransform(float2x2 PInA, FScreenTransform AToB)
{
	return dpv_add(dpv_mul(PInA, AToB.xy), AToB.zw);
}

#if PLATFORM_SUPPORTS_REAL_TYPES

CALL_SITE_DEBUGLOC 
half2x2 ApplyScreenTransform(half2x2 PInA, FScreenTransform AToB)
{
	return dpv_add(dpv_mul(PInA, half2(AToB.xy)), half2(AToB.zw));
}

#endif


/** Compute the group wave index into SGRP to then recompue the GroupThreadIndex latter */
CALL_SITE_DEBUGLOC 
uint GetGroupWaveIndex(uint GroupThreadIndex, uint GroupSize)
#if COMPILER_SUPPORTS_WAVE_ONCE
{
	uint LaneCountPerWave = WaveGetLaneCount();

	if (LaneCountPerWave >= GroupSize)
	{
		return 0;
	}

	return WaveReadLaneFirst(GroupThreadIndex / LaneCountPerWave);
}
#else
{
	return 0;
}
#endif

/** Force compute the group GroupThreadIndex through lane index and wave index if possible to reduce VGPR pressure. */
CALL_SITE_DEBUGLOC 
uint GetGroupThreadIndex(uint GroupThreadIndex, uint GroupWaveIndex)
#if COMPILER_SUPPORTS_WAVE_ONCE
{
	// shares GroupWaveOffset to save SALU
	uint GroupWaveOffset = WaveGetLaneCount() * GroupWaveIndex;

	// Do not share
	uint ComputedGroupThreadIndex;
	ISOLATE
	{
		ComputedGroupThreadIndex = GroupWaveOffset + WaveGetLaneIndex();
	}

	return ComputedGroupThreadIndex;
}
#else
{
	return GroupThreadIndex;
}
#endif

CALL_SITE_DEBUGLOC
tsr_short2x2 InvalidateOutputPixelPos(tsr_short2x2 PixelPos, uint2 ViewportMax)
#if 1
{
	tsr_short2x2 Subtract = dpv_sub(tsr_short2(ViewportMax - 1), PixelPos);
	tsr_ushort2 Override = tsr_ushort2(Subtract[0] | Subtract[1]);

	#if CONFIG_COMPILE_FP16
		PixelPos[0] |= -tsr_short2((Override & uint16_t(0x8000)) >> 15);
	#else
		PixelPos[0] |= -tsr_short2((Override & uint(0x80000000)) >> 31);
	#endif

	return PixelPos;
}
#else
{
	bool bIsValidPixel = all(PixelPos < ViewportMax);
	PixelPos.x = bIsValidPixel ? PixelPos.x : ~tsr_short(0);
	return PixelPos;
}
#endif

CALL_SITE_DEBUGLOC
tsr_short2 InvalidateOutputPixelPos(tsr_short2 PixelPos, uint2 ViewportMax)
#if 1
{
	tsr_short2 Subtract = tsr_short2(ViewportMax - 1) - PixelPos;
	tsr_ushort Override = tsr_ushort(Subtract.x | Subtract.y);

	#if CONFIG_COMPILE_FP16
		PixelPos.x |= -tsr_short((Override & uint16_t(0x8000)) >> 15);
	#else
		PixelPos.x |= -tsr_short((Override & uint(0x80000000)) >> 31);
	#endif

	return PixelPos;
}
#else
{
	bool bIsValidPixel = all(PixelPos < ViewportMax);
	PixelPos.x = bIsValidPixel ? PixelPos.x : ~tsr_short(0);
	return PixelPos;
}
#endif

CALL_SITE_DEBUGLOC
tsr_ushort2 Map8x8Tile2x2Lane(uint GroupThreadIndex)
{
	tsr_ushort2 T = tsr_ushort(GroupThreadIndex).xx;
	tsr_ushort2 GroupId = (T >> tsr_ushort2(0, 1) & tsr_ushort2(0x01, 0x01)) | ((T >> tsr_ushort2(2 - 1, 4 - 1)) & tsr_ushort2(0x03 << 1, 0x03 << 1));
	return GroupId;
}

CALL_SITE_DEBUGLOC
tsr_ushort2 Map16x16Tile2x2Lane(uint GroupThreadIndex)
{
	tsr_ushort T = tsr_ushort(GroupThreadIndex);

	tsr_ushort2 GroupId;
	GroupId.x = ((T >> tsr_ushort(0)) & tsr_ushort(0x01)) | ((T >> tsr_ushort(2 - 1)) & tsr_ushort(0x07 << 1));
	GroupId.y = ((T >> tsr_ushort(1)) & tsr_ushort(0x01)) | ((T >> tsr_ushort(5 - 1)) & tsr_ushort(0x07 << 1));
	
	return GroupId;
}

CALL_SITE_DEBUGLOC
float SafeRcp(float x)
{
	return x > 0.0 ? rcp(x) : 0.0;
}

CALL_SITE_DEBUGLOC
float2 SafeRcp(float2 x)
{
	return float2(SafeRcp(x.x), SafeRcp(x.y));
}

CALL_SITE_DEBUGLOC
float3 SafeRcp(float3 x)
{
	return float3(SafeRcp(x.x), SafeRcp(x.y), SafeRcp(x.z));
}

CALL_SITE_DEBUGLOC
float4 SafeRcp(float4 x)
{
	return float4(SafeRcp(x.x), SafeRcp(x.y), SafeRcp(x.z), SafeRcp(x.w));
}

#if CONFIG_COMPILE_FP16

CALL_SITE_DEBUGLOC
half SafeRcp(half x)
#if PLATFORM_GPU_ARCH >= PLATFORM_GPU_ARCH_AMD_RDNA_2 && PLATFORM_GPU_ARCH <= PLATFORM_GPU_ARCH_AMD_LATTEST
{
	// If x=0.0, then MaxHalfFloat * 0.0 = 0.0
	return min(rcp(x), half(MaxHalfFloat)) * saturate(x * asfloat16(uint16_t(0x7C00)));
}
#else
{
	return select(x > half(0.0), min(rcp(x), half(MaxHalfFloat)), half(0.0));
}
#endif

CALL_SITE_DEBUGLOC
half2 SafeRcp(half2 x)
#if PLATFORM_GPU_ARCH >= PLATFORM_GPU_ARCH_AMD_RDNA_2 && PLATFORM_GPU_ARCH <= PLATFORM_GPU_ARCH_AMD_LATTEST
{
	// If x=0.0, then MaxHalfFloat * 0.0 = 0.0
	return min(rcp(x), half(MaxHalfFloat)) * saturate(x * asfloat16(uint16_t(0x7C00)));
}
#else
{
	return select(x > half(0.0), min(rcp(x), half(MaxHalfFloat)), half(0.0));
}
#endif

CALL_SITE_DEBUGLOC
half3 SafeRcp(half3 x)
#if PLATFORM_GPU_ARCH >= PLATFORM_GPU_ARCH_AMD_RDNA_2 && PLATFORM_GPU_ARCH <= PLATFORM_GPU_ARCH_AMD_LATTEST
{
	// If x=0.0, then MaxHalfFloat * 0.0 = 0.0
	return min(rcp(x), half(MaxHalfFloat)) * saturate(x * asfloat16(uint16_t(0x7C00)));
}
#else
{
	return select(x > half(0.0), min(rcp(x), half(MaxHalfFloat)), half(0.0));
}
#endif

CALL_SITE_DEBUGLOC
half4 SafeRcp(half4 x)
#if PLATFORM_GPU_ARCH >= PLATFORM_GPU_ARCH_AMD_RDNA_2 && PLATFORM_GPU_ARCH <= PLATFORM_GPU_ARCH_AMD_LATTEST
{
	// If x=0.0, then MaxHalfFloat * 0.0 = 0.0
	return min(rcp(x), half(MaxHalfFloat)) * saturate(x * asfloat16(uint16_t(0x7C00)));
}
#else
{
	return select(x > half(0.0), min(rcp(x), half(MaxHalfFloat)), half(0.0));
}
#endif

#endif

CALL_SITE_DEBUGLOC
tsr_half2x2 WeightedLerpFactors(tsr_half2 WeightA, tsr_half2 WeightB, tsr_half2 Blend)
{
	tsr_half2 BlendA = (tsr_half(1.0) - Blend) * WeightA;
	tsr_half2 BlendB = Blend * WeightB;
	tsr_half2 RcpBlend = SafeRcp(BlendA + BlendB);
	BlendA *= RcpBlend;
	BlendB *= RcpBlend;
	return tsr_half2x2(BlendA, BlendB);
}

CALL_SITE_DEBUGLOC
tsr_half2 WeightedLerpFactors(tsr_half WeightA, tsr_half WeightB, tsr_half Blend)
{
	return dpv_lo(WeightedLerpFactors(
		dpv_interleave_mono_registers(WeightA),
		dpv_interleave_mono_registers(WeightB),
		dpv_interleave_mono_registers(Blend)));
}

CALL_SITE_DEBUGLOC
bool TakeOnlyOneSamplePair(float2 Offset)
{
	return Offset.y > 0.0 || (Offset.x > 0.0 && Offset.y == 0.0);
}

tsr_half ComputePredictionCompleteness(tsr_half SampleHistoryValidity)
{
	return saturate(SampleHistoryValidity * tsr_half(MAX_SAMPLE_COUNT) - tsr_half(0.2));
}

CALL_SITE_DEBUGLOC
void IsOffScreenOrDisoccluded(uint bCameraCut, float2x2 ScreenPos, bool2 bIsParallaxRejected, out bool2 bIsOffScreen, out bool2 bIsDisoccluded)
{
	bool bIsCameraCut = bCameraCut != 0;
	bool2 bIsOutOfBounds = max(abs(ScreenPos[0]), abs(ScreenPos[1])) >= 1.0;

	bIsOffScreen = or(bIsCameraCut, bIsOutOfBounds);
	bIsDisoccluded = and(!bIsOffScreen, bIsParallaxRejected);
}

tsr_half2 ComputePrevWeightMultiplier(tsr_half2 BlendFactor)
{
	return (1.0 - BlendFactor) / BlendFactor;
}

tsr_half2 RejectionFactorToBlendFactor(tsr_half2 RejectionFactor)
{
	return tsr_half(1.0) - RejectionFactor;
}
