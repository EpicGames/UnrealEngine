// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRKernels.ush"
#include "/Engine/Public/LaneVectorization.ush"

/** Additional functions for LaneVectorization used in TSR */
TVECTOR_FUNCTION_1PARAMS(ScalarType, SafeRcp)

//-------------------------------------------------------

#ifndef WAVE_COUNT_X
	#error WAVE_COUNT_X is undefined.
#endif

#ifndef WAVE_COUNT_Y
	#error WAVE_COUNT_Y is undefined.
#endif

#ifndef LANE_COUNT_X
	#error LANE_COUNT_X is undefined.
#endif

#ifndef LANE_COUNT_Y
	#error LANE_COUNT_Y is undefined.
#endif

/** Total number of waves. */
#define WAVE_COUNT (WAVE_COUNT_X * WAVE_COUNT_Y)

/** Total number of lane per wave. */
#define LANE_COUNT (LANE_COUNT_X * LANE_COUNT_Y)


/**
 * LANE_COUNT_X=2
 * LANE_COUNT_Y=2
 * LaneStride=(4, 2)
 *
 *    o   o   o   o | o   o   o   o
 *	                |
 *    o   o   o   o | o   o   o   o
 *	  --------------+--------------
 *    o   o   o   o | o   o   o   o
 *	                |
 *    o   o   o   o | o   o   o   o
 */


//------------------------------------------------------- VECTOR MEMORY ORDER

/** Returns coordinate of a pixel within a lane. */
CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
tsr_short2 GetSimdIndexPixelCoordinateInLane(const uint SimdIndex)
{
	return tsr_short2(SimdIndex % LaneStrideX, SimdIndex / LaneStrideX);
}

CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
uint GetPixelCoordinateInLaneSimdIndex(const tsr_short2 PixelCoordinateInLane)
{
	return uint(dot(PixelCoordinateInLane, tsr_short2(1, LaneStrideX)));
}


/** Returns coordinate of a pixel within a wave. */
CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
tsr_short2 GetLaneSimdPixelOffset(const uint LaneIndex, const uint SimdIndex)
{
	return (
		tsr_short2(LaneIndex % uint(LANE_COUNT_X * WAVE_COUNT_X), LaneIndex / uint(LANE_COUNT_X * WAVE_COUNT_X)) * tsr_short2(LaneStrideX, LaneStrideY) +
		GetSimdIndexPixelCoordinateInLane<LaneStrideX, LaneStrideY>(SimdIndex)
	);
}

/** Returns coordinate of a element within a lane. */
CALL_SITE_DEBUGLOC
tsr_short2 GetElementPixelPosInLane(const uint LaneStrideX, const uint LaneStrideY, const uint ElementIndex)
{
	return tsr_short2(ElementIndex % LaneStrideX, ElementIndex / LaneStrideX);
}

/** Returns coordinate of a element within a tile. */
CALL_SITE_DEBUGLOC
tsr_short2 GetElementPixelOffsetInTile(const uint LaneStrideX, const uint LaneStrideY, const uint LaneIndex, const uint SimdIndex)
{
	return (
		tsr_short2(LaneIndex % uint(LANE_COUNT_X * WAVE_COUNT_X), LaneIndex / uint(LANE_COUNT_X * WAVE_COUNT_X)) * tsr_short2(LaneStrideX, LaneStrideY) +
		GetElementPixelPosInLane(LaneStrideX, LaneStrideY, SimdIndex)
	);
}

CALL_SITE_DEBUGLOC
tsr_short2 ComputeElementInputPixelPos(
	const uint2 LaneStride,
	const uint2 TileOverscan,
	const uint2 PixelViewportMin,
	const uint2 PixelViewportMax,
	const uint2 GroupId,
	const uint  GroupThreadIndex,
	const uint  InputElementIndex,
	const int2  InputDataOffset = int2(0, 0),
	const uint  InputResDivisor = 1)
{
	const uint2 TileSize = uint2(LaneStride.x * WAVE_COUNT_X * LANE_COUNT_X, LaneStride.y * WAVE_COUNT_Y * LANE_COUNT_Y);
	const uint2 TileStride = uint2(TileSize - 2 * TileOverscan);
	
	tsr_short2 InputPixelOffsetMin = tsr_short2(PixelViewportMin / InputResDivisor);
	tsr_short2 InputPixelOffsetMax = tsr_short2((PixelViewportMax + (InputResDivisor - 1)) / InputResDivisor - 1);
	
	tsr_short2 InputTileOffset = tsr_short2(TileStride / InputResDivisor) * tsr_short2(GroupId) - tsr_short2(TileOverscan / InputResDivisor);
	tsr_short2 InputPixelOffsetInTile = GetElementPixelOffsetInTile(
		LaneStride.x / InputResDivisor,
		LaneStride.y / InputResDivisor,
		GroupThreadIndex,
		InputElementIndex);

	return fastClamp(InputPixelOffsetMin + InputTileOffset + InputPixelOffsetInTile, InputPixelOffsetMin, InputPixelOffsetMax);
}

CALL_SITE_DEBUGLOC
tsr_short2 ComputeElementOutputPixelPos(
	const uint2 LaneStride,
	const uint2 TileOverscan,
	const uint2 PixelViewportMin,
	const uint2 PixelViewportMax,
	const uint2 GroupId,
	const uint  GroupThreadIndex,
	const uint  OutputElementIndex,
	const int2  OutputDataOffset = int2(0, 0),
	const uint  OutputResDivisor = 1,
	const bool  bDebugDisablePadding = false)
{
	const uint2 TileSize = uint2(LaneStride.x * WAVE_COUNT_X * LANE_COUNT_X, LaneStride.y * WAVE_COUNT_Y * LANE_COUNT_Y);
	const uint2 TileStride = bDebugDisablePadding ? TileSize : uint2(TileSize - 2 * TileOverscan);
	
	tsr_short2 OutputPixelOffsetMin = tsr_short2(PixelViewportMin / OutputResDivisor);
	tsr_short2 OutputPixelOffsetMax = tsr_short2((PixelViewportMax + (OutputResDivisor - 1)) / OutputResDivisor);

	const uint2 DataMinInTile = TileOverscan - uint2(OutputDataOffset);
	const uint2 DataMaxInTile = TileSize - uint2(OutputDataOffset) - TileOverscan;
	
	tsr_short2 OutputTileOffset = tsr_short2(TileStride / OutputResDivisor) * tsr_short2(GroupId) - tsr_short2((bDebugDisablePadding ? 0u : TileOverscan) / OutputResDivisor);
	tsr_short2 OutputPixelOffsetInTile = GetElementPixelOffsetInTile(
		LaneStride.x / OutputResDivisor,
		LaneStride.y / OutputResDivisor,
		GroupThreadIndex,
		OutputElementIndex);

	tsr_short2 OutputPixelOffsetInOutputTexture = OutputPixelOffsetMin + OutputTileOffset + OutputPixelOffsetInTile;
	
	bool2 bIsWithinOutputViewport = OutputPixelOffsetInOutputTexture < OutputPixelOffsetMax;
	bool2 bIsNotPadding = or(bDebugDisablePadding.xx, and(
		OutputPixelOffsetInTile * tsr_short(OutputResDivisor) >= tsr_short2(DataMinInTile),
		OutputPixelOffsetInTile * tsr_short(OutputResDivisor) <  tsr_short2(DataMaxInTile)));

	bool2 bIsValid = and(bIsNotPadding, bIsWithinOutputViewport);

	OutputPixelOffsetInOutputTexture = select(bIsValid, OutputPixelOffsetInOutputTexture, ~tsr_short(0).xx);

	return OutputPixelOffsetInOutputTexture;
}


//------------------------------------------------------- PADDING

CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<bool, 1, LaneStrideX, LaneStrideY> IsValidTilePixel(const uint2 TileOverscan)
{
	const uint2 TileSize = uint2(LaneStrideX * WAVE_COUNT_X * LANE_COUNT_X, LaneStrideY * WAVE_COUNT_Y * LANE_COUNT_Y);

	TLaneVector2D<bool, 1, LaneStrideX, LaneStrideY> bIsValidTilePixel;

	UNROLL
	for (uint ElementIndex = 0; ElementIndex < LaneStrideX * LaneStrideY; ElementIndex++)
	{
		tsr_short2 PixelOffsetInTile = GetElementPixelOffsetInTile(
			LaneStrideX,
			LaneStrideY,
			GGroupThreadIndex,
			ElementIndex);
			
		bool2 bIsNotPadding = and(
			PixelOffsetInTile >= tsr_short2(TileOverscan),
			PixelOffsetInTile <  tsr_short2(TileSize - TileOverscan));

		bIsValidTilePixel.SetElement(ElementIndex, all(bIsNotPadding));
	}

	return bIsValidTilePixel;
}


//------------------------------------------------------- VOTE

groupshared uint SharedAtomic;

CALL_SITE_DEBUGLOC
bool AnyThread(bool X)
#if WAVE_COUNT_X == 1 && WAVE_COUNT_Y == 1 && COMPILER_SUPPORTS_WAVE_VOTE
{
	return WaveActiveAnyTrue(X);
}
#elif COMPILER_SUPPORTS_WAVE_VOTE && PLATFORM_GPU_ARCH == PLATFORM_GPU_ARCH_AMD_RDNA_2
{
	const uint kLaneCount = WaveGetLaneCount();
	const uint kThreadCount = LANE_COUNT_X * LANE_COUNT_Y * WAVE_COUNT_X * WAVE_COUNT_Y;
	const uint kWaveCount = kThreadCount / kLaneCount;

	bool AnyX = WaveActiveAnyTrue(X);

	GroupMemoryBarrierWithGroupSync();

	if (WaveIsFirstLane())
	{
		WriteDwordToLDS(GGroupThreadIndex / kLaneCount, select(AnyX, 1u, 0u));
	}
		
	GroupMemoryBarrierWithGroupSync();
		
	uint WaveX = ReadDwordFromLDS(GGroupThreadIndex % kWaveCount);

	return WaveActiveAnyTrue(WaveX != 0);
}
#elif defined(LDS_SIZE)
{
	const uint SharedIndex = 0;

	GroupMemoryBarrierWithGroupSync();
	
	if (GGroupThreadIndex == 0)
	{
		WriteDwordToLDS(SharedIndex, 0);
	}

	GroupMemoryBarrierWithGroupSync();
	
	AtomicIncrementLDSDword(SharedIndex, select(X, 1u, 0u));
	
	GroupMemoryBarrierWithGroupSync();
	
	return ReadDwordFromLDS(SharedIndex) > 0;
}
#else
{
	GroupMemoryBarrierWithGroupSync();
	
	if (GGroupThreadIndex == 0)
	{
		SharedAtomic = 0;
	}

	GroupMemoryBarrierWithGroupSync();
	
	InterlockedAdd(/* inout */ SharedAtomic, select(X, 1u, 0u));
	
	GroupMemoryBarrierWithGroupSync();
	
	return SharedAtomic > 0;
}
#endif


//------------------------------------------------------- ACCESS NEIGHBOR USING WAVE OPS

CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
uint GetNeighborElementIndex(const uint ElementIndex, const tsr_short2 Offset)
{
	const uint2 ElementPixelCoord = uint2(GetSimdIndexPixelCoordinateInLane<LaneStrideX, LaneStrideY>(ElementIndex));

	const uint2 NeigborElementPixelCoord = (ElementPixelCoord + uint2(Offset)) % uint2(LaneStrideX, LaneStrideY);
	const uint NeigborElementIndex = dot(NeigborElementPixelCoord, uint2(1, LaneStrideX));

	return NeigborElementIndex;
}

#if WAVE_COUNT_X == 1 || WAVE_COUNT_Y == 1

CALL_SITE_DEBUGLOC
template<uint LaneStrideX, uint LaneStrideY>
FWaveBroadcastSettings GetWaveBroadcastSettingsForNeighbor(const uint ElementIndex, const tsr_short2 Offset)
{
	const uint2 ElementPixelCoord = uint2(GetSimdIndexPixelCoordinateInLane<LaneStrideX, LaneStrideY>(ElementIndex));

	const bool2 bNeedsLaneRotation = (ElementPixelCoord + uint2(Offset)) >= uint2(LaneStrideX, LaneStrideY);
	const int LaneRotation = dot(select(bNeedsLaneRotation, int2(Offset), 0), int2(1, LANE_COUNT_X));

	return InitWaveRotateLaneGroup(/* LaneGroupSize = */ LANE_COUNT, LaneRotation);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
vector<ScalarType, VectorSize> WaveAccessNeighborElement(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	const uint ElementIndex,
	const tsr_short2 Offset)
{
	const uint NeigborElementIndex = GetNeighborElementIndex<LaneStrideX, LaneStrideY>(ElementIndex, Offset);
	const FWaveBroadcastSettings BroadcastSettings = GetWaveBroadcastSettingsForNeighbor<LaneStrideX, LaneStrideY>(ElementIndex, Offset);
		
	// Access the lement.
	vector<ScalarType, VectorSize> ReturnSimdElement;
	if (BroadcastSettings.Rotate != 0)
	{
		ReturnSimdElement = Center.Registers.WaveBroadcastElement(BroadcastSettings, NeigborElementIndex);
	}
	else
	{
		ReturnSimdElement = Center.GetElement(NeigborElementIndex);
	}
	return ReturnSimdElement;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> WaveAccessNeighborTexel(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	const tsr_short2 Offset)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Return;
	
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < LaneStrideX * LaneStrideY; ElementIndex++)
	{
		Return.SetElement(ElementIndex, WaveAccessNeighborElement(Center, ElementIndex, Offset));
	}

	Return.TightenRegisters();
	return Return;
}

#endif // WAVE_COUNT_X == 1 || WAVE_COUNT_Y == 1


//------------------------------------------------------- 3x1 NEIGHBOR ACCESS

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void AccessNeighborTexels3x1(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN)
#if WAVE_COUNT_X > 1
{
	const uint TotalNumberOfLaneInGroup = LANE_COUNT_Y * WAVE_COUNT;

	if (LaneStrideX == 1)
	{
		GroupMemoryBarrierWithGroupSync();
		UNROLL
		for (uint SimdIndex = 0; SimdIndex < LaneStrideY; SimdIndex++)
		{
			WriteVectorToLDS<ScalarType, VectorSize>(GGroupThreadIndex + SimdIndex * TotalNumberOfLaneInGroup, CO.GetElement(SimdIndex));
		}
		GroupMemoryBarrierWithGroupSync();
		
		UNROLL
		for (uint SimdIndex = 0; SimdIndex < LaneStrideY; SimdIndex++)
		{
			vector<ScalarType, VectorSize> FirstElement;
			vector<ScalarType, VectorSize> LastElement;
			ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex - 1) % TotalNumberOfLaneInGroup) + SimdIndex * TotalNumberOfLaneInGroup, /* out */ FirstElement);
			ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex + 1) % TotalNumberOfLaneInGroup) + SimdIndex * TotalNumberOfLaneInGroup, /* out */ LastElement);
	
			CP.SetElement(SimdIndex, LastElement);
			CN.SetElement(SimdIndex, FirstElement);
		}
	}
	else
	{
		// TODO
		CP = CN = CO;
	}

	CP.TightenRegisters();
	CN.TightenRegisters();
}
#else
{
	CP = WaveAccessNeighborTexel(CO, tsr_short2(1, 0));
	CN = WaveAccessNeighborTexel(CO, tsr_short2(-1, 0));
}
#endif


//------------------------------------------------------- 1x3 NEIGHBOR ACCESS

CALL_SITE_DEBUGLOC
ISOLATE
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void Write1x3CenterToLDS(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO)
#if WAVE_COUNT_Y > 1
{
	const uint TotalNumberOfLaneInGroup = LANE_COUNT_X * WAVE_COUNT;

	GroupMemoryBarrierWithGroupSync();

	if (LaneStrideY == 1)
	{
		UNROLL
		for (uint ElementIndex = 0; ElementIndex < LaneStrideX; ElementIndex++)
		{
			WriteVectorToLDS<ScalarType, VectorSize>(GGroupThreadIndex + ElementIndex * TotalNumberOfLaneInGroup, CO.GetElement(ElementIndex));
		}
	}
	else
	{
		const uint FirstElementIndex = 0;
		const uint LastElementIndex = LaneStrideX * LaneStrideY - LaneStrideX;

		UNROLL
		for (uint ElementIndex = 0; ElementIndex < LaneStrideX; ElementIndex++)
		{
			WriteVectorToLDS<ScalarType, VectorSize>(GGroupThreadIndex + (0 + ElementIndex) * TotalNumberOfLaneInGroup, CO.GetElement(FirstElementIndex + ElementIndex));
			WriteVectorToLDS<ScalarType, VectorSize>(GGroupThreadIndex + (LaneStrideX + ElementIndex) * TotalNumberOfLaneInGroup, CO.GetElement(LastElementIndex + ElementIndex));
		}
	}

	GroupMemoryBarrierWithGroupSync();
}
#else
{
	// NOP
}
#endif

CALL_SITE_DEBUGLOC
ISOLATE
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void Read1x3NeighborElementsFromLDS(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO,
	const uint ElementIndex,
	out vector<ScalarType, VectorSize> CP,
	out vector<ScalarType, VectorSize> CN)
#if WAVE_COUNT_Y > 1
{
	const uint TotalNumberOfLaneInGroup = LANE_COUNT_X * WAVE_COUNT;

	if (LaneStrideY == 1)
	{
		ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex - LANE_COUNT_X * WAVE_COUNT_X) % TotalNumberOfLaneInGroup) + ElementIndex * TotalNumberOfLaneInGroup, /* out */ CN);
		ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex + LANE_COUNT_X * WAVE_COUNT_X) % TotalNumberOfLaneInGroup) + ElementIndex * TotalNumberOfLaneInGroup, /* out */ CP);
	}
	else // if (LaneStrideY > 1)
	{
		if (ElementIndex < LaneStrideX)
		{
			ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex - LANE_COUNT_X * WAVE_COUNT_X) % TotalNumberOfLaneInGroup) + (LaneStrideX + ElementIndex) * TotalNumberOfLaneInGroup, /* out */ CN);
		}
		else
		{
			CN = CO.GetElement(ElementIndex - LaneStrideX);
		}
		
		const uint LastElementIndex = LaneStrideX * LaneStrideY - LaneStrideX;
		if (ElementIndex >= LastElementIndex)
		{
			ReadVectorFromLDS<ScalarType, VectorSize>(((GGroupThreadIndex + LANE_COUNT_X * WAVE_COUNT_X) % TotalNumberOfLaneInGroup) + (0 + ElementIndex - LastElementIndex) * TotalNumberOfLaneInGroup, /* out */ CP);
		}
		else
		{
			CP = CO.GetElement(ElementIndex + LaneStrideX);
		}
	}
}
#else
{
	CP = WaveAccessNeighborElement(CO, ElementIndex, tsr_short2(0, 1));
	CN = WaveAccessNeighborElement(CO, ElementIndex, tsr_short2(0, -1));
}
#endif

CALL_SITE_DEBUGLOC
ISOLATE
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void Read1x3CenterFromLDS(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN)
{
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < LaneStrideX * LaneStrideY; ElementIndex++)
	{
		vector<ScalarType, VectorSize> CPElement;
		vector<ScalarType, VectorSize> CNElement;
		Read1x3NeighborElementsFromLDS(CO, ElementIndex, /* out */ CPElement, /* out */ CNElement);

		CP.SetElement(ElementIndex, CPElement);
		CN.SetElement(ElementIndex, CNElement);
	}
	
	CP.TightenRegisters();
	CN.TightenRegisters();
}

CALL_SITE_DEBUGLOC
ISOLATE
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void AccessNeighborTexels1x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN)
{
	Write1x3CenterToLDS(CO);
	Read1x3CenterFromLDS(CO, /* out */ CP, /* out */ CN);
}


//------------------------------------------------------- CONVOLUTION WITH SINK FOR LOWER REGISTER PRESSURE

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY, typename ConvolutionSinkType>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Convolve1x3Then3x1FromLDS(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> C, ConvolutionSinkType ConvolutionSink)
#if WAVE_COUNT_X == 1 && 0
{
	// Only if LaneStrideX==1 and WAVE_COUNT_X==1
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> R;
	
	UNROLL
	for (uint ElementIndex = 0; ElementIndex < LaneStrideX * LaneStrideY; ElementIndex++)
	{
		vector<ScalarType, VectorSize> CPElement;
		vector<ScalarType, VectorSize> CNElement;
		Read1x3NeighborElementsFromLDS(C, ElementIndex, /* out */ CPElement, /* out */ CNElement);

		vector<ScalarType, VectorSize> HCElement = ConvolutionSink.ConvolveElements1x3(CNElement, C.GetElement(ElementIndex), CPElement);

		const FWaveBroadcastSettings HPSettings = GetWaveBroadcastSettingsForNeighbor<LaneStrideX, LaneStrideY>(ElementIndex, tsr_short2(1, 0));
		const FWaveBroadcastSettings HNSettings = GetWaveBroadcastSettingsForNeighbor<LaneStrideX, LaneStrideY>(ElementIndex, tsr_short2(-1, 0));
		
		vector<ScalarType, VectorSize> HPElement = WaveBroadcast(HPSettings, HCElement);
		vector<ScalarType, VectorSize> HNElement = WaveBroadcast(HNSettings, HCElement);

		R.SetElement(ElementIndex, ConvolutionSink.ConvolveElements3x1(HNElement, HCElement, HPElement));
	}
	
	return R;
}
#else
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> VO;
	{
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = C;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
		Read1x3CenterFromLDS(CO, /* out */ CP, /* out */ CN);

		UNROLL
		for (uint RegisterRowIndex = 0; RegisterRowIndex < TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::RegisterRowCount; RegisterRowIndex++)
		{
			VO.Registers.SetRegisterRow(RegisterRowIndex, ConvolutionSink.ConvolveRegisterRows1x3(CN.Registers.GetRegisterRow(RegisterRowIndex), CO.Registers.GetRegisterRow(RegisterRowIndex), CP.Registers.GetRegisterRow(RegisterRowIndex)));
		}
	}
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> R;
	{
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = VO;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
		AccessNeighborTexels3x1(CO, /* out */ CP, /* out */ CN);

		UNROLL
		for (uint RegisterRowIndex = 0; RegisterRowIndex < TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::RegisterRowCount; RegisterRowIndex++)
		{
			R.Registers.SetRegisterRow(RegisterRowIndex, ConvolutionSink.ConvolveRegisterRows3x1(CN.Registers.GetRegisterRow(RegisterRowIndex), CO.Registers.GetRegisterRow(RegisterRowIndex), CP.Registers.GetRegisterRow(RegisterRowIndex)));
		}
	}

	return R;
}
#endif

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY, typename ConvolutionSinkType>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Convolve1x3Then3x1(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center, ConvolutionSinkType ConvolutionSink)
{
	Write1x3CenterToLDS(Center);
	return Convolve1x3Then3x1FromLDS(Center, ConvolutionSink);
}


//------------------------------------------------------- BLUR 3x3 CONVOLUTION

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> WeightedSum3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center, float3 HorizontalWeights, float3 VerticalWeights)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BO = (
		CO * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(VerticalWeights.y)) +
		CP * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(VerticalWeights.z)) +
		CN * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(VerticalWeights.x)));

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BN;
	AccessNeighborTexels3x1(BO, /* out */ BP, /* out */ BN);

	return (
		BO * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(HorizontalWeights.y)) +
		BP * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(HorizontalWeights.z)) +
		BN * TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(ScalarType(HorizontalWeights.x)));
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Sum3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	return WeightedSum3x3(Center, /* HorizontalWeights = */ float(1.0).xxx, /* VerticalWeights = */ float(1.0).xxx);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> WeightedAvg3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center, float3 NormalizedWeights)
{
	return WeightedSum3x3(Center, /* HorizontalWeights = */ NormalizedWeights, /* VerticalWeights = */ NormalizedWeights);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Blur3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	return WeightedAvg3x3(Center, /* NormalizedWeights = */ float3(0.25, 0.5, 0.25));
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BlurEven3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	return WeightedAvg3x3(Center, /* NormalizedWeights = */ float3(0.3333, 0.3333, 0.3333));
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Sharpen3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	return WeightedAvg3x3(Center, /* NormalizedWeights = */ float3(-0.25, 1.5, -0.25));
}


//------------------------------------------------------- TOTAL VARIATION 3x3 CONVOLUTION

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> TotalVariation3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
#if 0
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Sum = Blur3x3(Center);
	
	return TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(1) * Center - Sum;
}
#else
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Sum = WeightedSum3x3(Center, /* HorizontalWeights = */ float(1.0).xxx, /* VerticalWeights = */ float3(0.125, 0.125, 0.125));

	return TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(1.125) * Center - Sum;
}
#endif


//------------------------------------------------------- MIN & MAX 3x3 CONVOLUTION

template<typename ScalarType, uint VectorSize, uint InSimdSizeX, uint InSimdSizeY>
struct TConvolutionSinkMin
{
	static const uint RegisterRowSize = TLaneVector2D<ScalarType, VectorSize, InSimdSizeX, InSimdSizeY>::RegisterRowSize;

	uint Unused;

	vector<ScalarType, VectorSize> ConvolveElements3x1(vector<ScalarType, VectorSize> N, vector<ScalarType, VectorSize> C, vector<ScalarType, VectorSize> P)
	{
		return min3(N, C, P);
	}

	vector<ScalarType, VectorSize> ConvolveElements1x3(vector<ScalarType, VectorSize> N, vector<ScalarType, VectorSize> C, vector<ScalarType, VectorSize> P)
	{
		return min3(N, C, P);
	}
	
	vector<ScalarType, RegisterRowSize> ConvolveRegisterRows3x1(vector<ScalarType, RegisterRowSize> N, vector<ScalarType, RegisterRowSize> C, vector<ScalarType, RegisterRowSize> P)
	{
		return min3(N, C, P);
	}

	vector<ScalarType, RegisterRowSize> ConvolveRegisterRows1x3(vector<ScalarType, RegisterRowSize> N, vector<ScalarType, RegisterRowSize> C, vector<ScalarType, RegisterRowSize> P)
	{
		return min3(N, C, P);
	}
};

template<typename ScalarType, uint VectorSize, uint InSimdSizeX, uint InSimdSizeY>
struct TConvolutionSinkMax
{
	static const uint RegisterRowSize = TLaneVector2D<ScalarType, VectorSize, InSimdSizeX, InSimdSizeY>::RegisterRowSize;

	uint Unused;

	vector<ScalarType, VectorSize> ConvolveElements3x1(vector<ScalarType, VectorSize> N, vector<ScalarType, VectorSize> C, vector<ScalarType, VectorSize> P)
	{
		return max3(N, C, P);
	}

	vector<ScalarType, VectorSize> ConvolveElements1x3(vector<ScalarType, VectorSize> N, vector<ScalarType, VectorSize> C, vector<ScalarType, VectorSize> P)
	{
		return max3(N, C, P);
	}
	
	vector<ScalarType, RegisterRowSize> ConvolveRegisterRows3x1(vector<ScalarType, RegisterRowSize> N, vector<ScalarType, RegisterRowSize> C, vector<ScalarType, RegisterRowSize> P)
	{
		return max3(N, C, P);
	}

	vector<ScalarType, RegisterRowSize> ConvolveRegisterRows1x3(vector<ScalarType, RegisterRowSize> N, vector<ScalarType, RegisterRowSize> C, vector<ScalarType, RegisterRowSize> P)
	{
		return max3(N, C, P);
	}
};

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Min3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
#if 1
{
	TConvolutionSinkMin<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinSink = {0};
	return Convolve1x3Then3x1(Center, MinSink);
}
#else
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinO = min3(CN, CP, CO);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinN;
	AccessNeighborTexels3x1(MinO, /* out */ MinP, /* out */ MinN);

	return min3(MinN, MinP, MinO);
}
#endif

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Max3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
#if 1
{
	TConvolutionSinkMax<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxSink = {0};
	return Convolve1x3Then3x1(Center, MaxSink);
}
#else
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxO = max3(CN, CP, CO);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxN;
	AccessNeighborTexels3x1(MaxO, /* out */ MaxP, /* out */ MaxN);

	return max3(MaxN, MaxP, MaxO);
}
#endif

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> MaxRGBMinA3x3(TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> Center)
{
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> MaxO = max3(CN, CP, CO);
	MaxO.SetComponent(3, min3(CN[3], CP[3], CO[3]));

	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> MaxP;
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> MaxN;
	AccessNeighborTexels3x1(MaxO, /* out */ MaxP, /* out */ MaxN);
	
	TLaneVector2D<ScalarType, 4, LaneStrideX, LaneStrideY> R = max3(MaxN, MaxP, MaxO);
	R.SetComponent(3, min3(MaxN[3], MaxP[3], MaxO[3]));

	return R;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> MaxRMinG3x3(TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> Center)
{
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> MaxO = max3(CN, CP, CO);
	MaxO.SetComponent(1, min3(CN[1], CP[1], CO[1]));

	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> MaxP;
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> MaxN;
	AccessNeighborTexels3x1(MaxO, /* out */ MaxP, /* out */ MaxN);
	
	TLaneVector2D<ScalarType, 2, LaneStrideX, LaneStrideY> R = max3(MaxN, MaxP, MaxO);
	R.SetComponent(1, min3(MaxN[1], MaxP[1], MaxO[1]));

	return R;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void MinMax3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> OutMin,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> OutMax)
#if WAVE_COUNT_X == 1 && 1
{
	TConvolutionSinkMin<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinSink = {0};
	TConvolutionSinkMax<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxSink = {0};
	
	Write1x3CenterToLDS(Center);
	OutMin = Convolve1x3Then3x1FromLDS(Center, MinSink);
	OutMax = Convolve1x3Then3x1FromLDS(Center, MaxSink);
}
#elif 1
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CO = Center;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CN;
	AccessNeighborTexels1x3(CO, /* out */ CP, /* out */ CN);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinO = min3(CN, CP, CO);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxO = max3(CN, CP, CO);

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MinN;
	AccessNeighborTexels3x1(MinO, /* out */ MinP, /* out */ MinN);
	OutMin = min3(MinN, MinP, MinO);

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxP;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxN;
	AccessNeighborTexels3x1(MaxO, /* out */ MaxP, /* out */ MaxN);
	OutMax = max3(MaxN, MaxP, MaxO);
}
#else
{
	OutMin = Min3x3(Center);
	OutMax = Max3x3(Center);
}
#endif

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> MaxMinusMin3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoxMin;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoxMax;
	MinMax3x3(Center, /* out */ BoxMin, /* out */ BoxMax);
	return BoxMax - BoxMin;
}


//------------------------------------------------------- CLAMP 3x3 CONVOLUTIONS

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Clamp3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClamp,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryCenter)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMin;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMax;
	MinMax3x3(BoundaryCenter, /* out */ BoundaryMin, /* out */ BoundaryMax);

	return fastClamp(ToClamp, BoundaryMin, BoundaryMax);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> LerpClamp3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClamp,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Boundary /*Boundary inside min max of ToClamp)*/,
	TLaneVector2D<ScalarType, 1, LaneStrideX, LaneStrideY> LerpFactor)
{
    #if CONFIG_COMPILE_FP16
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMin;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMax;
		MinMax3x3(Boundary, /* out */ BoundaryMin, /* out */ BoundaryMax);

		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClampBoundaryMin;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClampBoundaryMax;
		MinMax3x3(ToClamp, /* out */ ToClampBoundaryMin, /* out */ ToClampBoundaryMax);

		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> LerpFactorVector = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Vectorize(LerpFactor);
		BoundaryMin = lerp(BoundaryMin, ToClampBoundaryMin, LerpFactorVector);
		BoundaryMax = lerp(BoundaryMax, ToClampBoundaryMax, LerpFactorVector);
    #else
		// register pressure is high
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMin;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMax;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryLimit;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClampBoundaryLimit;
		TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> LerpFactorVector = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Vectorize(LerpFactor);

		BoundaryLimit = Min3x3(Boundary);
		ToClampBoundaryLimit = Min3x3(ToClamp);
		BoundaryMin = lerp(BoundaryLimit, ToClampBoundaryLimit, LerpFactorVector);

		BoundaryLimit = Max3x3(Boundary);
		ToClampBoundaryLimit = Max3x3(ToClamp);
		BoundaryMax = lerp(BoundaryLimit, ToClampBoundaryLimit, LerpFactorVector);
    #endif

	return fastClamp(ToClamp, BoundaryMin, BoundaryMax);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> AnnihilateToGuide3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClamp,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Guide)
{
	return Clamp3x3(ToClamp, Clamp3x3(Guide, ToClamp));
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void AnnihilateMutually3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Input,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> History,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> AnnihilatedInput,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> AnnihilatedHistory)
{
	AnnihilatedInput   = AnnihilateToGuide3x3(Input  , /* Guide = */ History);
	AnnihilatedHistory = AnnihilateToGuide3x3(History, /* Guide = */ Input);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint LaneStrideX, uint LaneStrideY>
void AnnihilateMutuallySingleChannel3x3(
	TLaneVector2D<ScalarType, 1, LaneStrideX, LaneStrideY> Input,
	TLaneVector2D<ScalarType, 1, LaneStrideX, LaneStrideY> History,
	out TLaneVector2D<ScalarType, 1, LaneStrideX, LaneStrideY> AnnihilatedInput,
	out TLaneVector2D<ScalarType, 1, LaneStrideX, LaneStrideY> AnnihilatedHistory)
{
	Deconcatenate(AnnihilateToGuide3x3(/* ToClamp = */ Concatenate(Input, History), /* Guide = */ Concatenate(History, Input)), /* out */ AnnihilatedInput, /* out */ AnnihilatedHistory);
}


//------------------------------------------------------- MIN & MAX & CLAMP 3x3 PLUS CONVOLUTION

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
void MinMaxPlus3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> OutMin,
	out TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> OutMax)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CVP, CVN;
	AccessNeighborTexels1x3(Center, /* out */ CVP, /* out */ CVN);
	OutMin = min3(CVP, CVN, Center);
	OutMax = max3(CVP, CVN, Center);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> CHP, CHN;
	AccessNeighborTexels3x1(Center, /* out */ CHP, /* out */ CHN);
	OutMin = min3(CHN, CHP, OutMin);
	OutMax = max3(CHN, CHP, OutMax);
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ClampPlus3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> ToClamp,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryCenter)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMin;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> BoundaryMax;
	MinMaxPlus3x3(BoundaryCenter, /* out */ BoundaryMin, /* out */ BoundaryMax);

	return fastClamp(ToClamp, BoundaryMin, BoundaryMax);
}


//------------------------------------------------------- MEDIAN 3x3 CONVOLUTION

// Operator that output, lowest, median and highest values from 3 input values.
CALL_SITE_DEBUGLOC
template<typename FSampleType>
void LMHOperator(FSampleType A, FSampleType B, FSampleType C, out FSampleType L, out FSampleType M, out FSampleType H)
#if COMPILER_SUPPORTS_MED3
{
	L = min3(A, B, C);
	M = med3(A, B, C);
	H = max3(A, B, C);
}
#else
{
	FSampleType X = min(B, C);
	FSampleType Y = max(B, C);
	
	L = min(A, X);
	FSampleType Z = max(A, X);

	M = min(Z, Y);
	H = max(Z, Y);
}
#endif

// 3 samples median.
CALL_SITE_DEBUGLOC
template<typename FSampleType>
FSampleType Median(FSampleType A, FSampleType B, FSampleType C)
#if COMPILER_SUPPORTS_MED3
{
	return med3(A, B, C);
}
#else
{
	FSampleType L, M, H;
	LMHOperator(A, B, C, L, M, H);
	return M;
}
#endif

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Median3x3(TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> C0 = Center;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> C1;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> C2;
	AccessNeighborTexels1x3(C0, /* out */ C1, /* out */ C2);
	
	// First layer.
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0L0;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0M0;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0H0;
	LMHOperator(C0, C1, C2, M0L0, M0M0, M0H0);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0L1;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0L2;
	AccessNeighborTexels3x1(M0L0, /* out */ M0L1, /* out */ M0L2);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0M1;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0M2;
	AccessNeighborTexels3x1(M0M0, /* out */ M0M1, /* out */ M0M2);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0H1;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M0H2;
	AccessNeighborTexels3x1(M0H0, /* out */ M0H1, /* out */ M0H2);
	
	// Second layer.
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> M1[3];
	M1[0] = max3(M0L0, M0L1, M0L2);
	M1[1] = Median(M0M0, M0M1, M0M2);
	M1[2] = min3(M0H0, M0H1, M0H2);
	
	// Third layer.
	return Median(M1[0], M1[1], M1[2]);
}

CALL_SITE_DEBUGLOC
template<uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<bool, VectorSize, LaneStrideX, LaneStrideY> MedianBool3x3(TLaneVector2D<bool, VectorSize, LaneStrideX, LaneStrideY> Center)
{
	TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY> Zero = TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY>::Const(tsr_half(0.0));
	TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY> One = TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY>::Const(tsr_half(1.0));

	TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY> CenterHalf = select(
		Center,
		One,
		Zero);
	TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY> CenterSum = Sum3x3(CenterHalf);
	return CenterSum > TLaneVector2D<tsr_half, VectorSize, LaneStrideX, LaneStrideY>::Const(4.0);
}


//------------------------------------------------------- DOWNSAMPLE 2x2 CONVOLUTIONS

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> DownsampleMin2x2(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Input)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> OutMin;
	UNROLL
	for (uint OutputSimdIndex = 0; OutputSimdIndex < ((LaneStrideX * LaneStrideY) / 4); OutputSimdIndex++)
	{
		const uint2 OutputPos = GetSimdIndexPixelCoordinateInLane<LaneStrideX / 2, LaneStrideY / 2>(OutputSimdIndex);
		vector<ScalarType, VectorSize> Input0 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[0])));
		vector<ScalarType, VectorSize> Input1 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[1])));
		vector<ScalarType, VectorSize> Input2 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[2])));
		vector<ScalarType, VectorSize> Input3 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[3])));

		OutMin.SetElement(OutputSimdIndex, min(min(Input0, Input1), min(Input2, Input3)));
	}
	return OutMin;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> DownsampleMax2x2(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Input)
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> OutMax;
	UNROLL
	for (uint OutputSimdIndex = 0; OutputSimdIndex < ((LaneStrideX * LaneStrideY) / 4); OutputSimdIndex++)
	{
		const uint2 OutputPos = GetSimdIndexPixelCoordinateInLane<LaneStrideX / 2, LaneStrideY / 2>(OutputSimdIndex);
		vector<ScalarType, VectorSize> Input0 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[0])));
		vector<ScalarType, VectorSize> Input1 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[1])));
		vector<ScalarType, VectorSize> Input2 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[2])));
		vector<ScalarType, VectorSize> Input3 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[3])));

		OutMax.SetElement(OutputSimdIndex, max(max(Input0, Input1), max(Input2, Input3)));
	}
	return OutMax;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> DownsampleDot2x2(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Input, const ScalarType Weights[4])
{
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> OutMax;
	UNROLL
	for (uint OutputSimdIndex = 0; OutputSimdIndex < ((LaneStrideX * LaneStrideY) / 4); OutputSimdIndex++)
	{
		const uint2 OutputPos = GetSimdIndexPixelCoordinateInLane<LaneStrideX / 2, LaneStrideY / 2>(OutputSimdIndex);
		vector<ScalarType, VectorSize> Input0 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[0])));
		vector<ScalarType, VectorSize> Input1 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[1])));
		vector<ScalarType, VectorSize> Input2 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[2])));
		vector<ScalarType, VectorSize> Input3 = Input.GetElement(GetPixelCoordinateInLaneSimdIndex<LaneStrideX, LaneStrideY>(tsr_short2(OutputPos * 2 + Offsets2x2[3])));

		OutMax.SetElement(OutputSimdIndex, Input0 * Weights[0] + Input1 * Weights[1] + Input2 * Weights[2] + Input3 * Weights[3]);
	}
	return OutMax;
}

CALL_SITE_DEBUGLOC
template<typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX / 2, LaneStrideY / 2> DownsampleAvg2x2(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Input)
{
	const ScalarType Weights[4] = { ScalarType(0.25), ScalarType(0.25), ScalarType(0.25), ScalarType(0.25) };
	return DownsampleDot2x2(Input, Weights);
}
