// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRCommon.ush"


//------------------------------------------------------- DERIVED
	
#if DIM_WAVE_SIZE == 0 || DEBUG_DIGESTABLE
	/** Load an entire 32x32 with 1x1 worth of data by lane, only using LDS. */
	#define GROUP_TILE_SIZE 32

	#define WAVE_COUNT_X GROUP_TILE_SIZE
	#define WAVE_COUNT_Y GROUP_TILE_SIZE

	#define LANE_COUNT_X 1
	#define LANE_COUNT_Y 1

	#define LANE_STRIDE_X 1
	#define LANE_STRIDE_Y 1

	#define LDS_SIZE (GROUP_TILE_SIZE * GROUP_TILE_SIZE)

	#define TENSOR_REGISTER_LAYOUT FRegisters_S_OneVector

#elif DIM_WAVE_SIZE == 64 || DIM_WAVE_SIZE == 32 || DIM_WAVE_SIZE == 16
	/** Layouts wave in row major 32x32 tile so an entire row of pixels fits within a single wave, and waveops horizontally, but LDS vertically.
	 * Each lane worth of a 1x4 of data per lane to minimise to trade VGPR pressure for both number of wave ops and LDS.
	 */
	#define GROUP_TILE_SIZE 32

	#if DIM_WAVE_SIZE == 16
		// We need a LANE_STRIDE_X of at least 2 to have the entire row of pixels fits within a single wave.
		#define LANE_STRIDE_X 2

		// Only process 2 pixels per lane to reduce register pressure on Intel Arc Alchemist GPUs
		#define LANE_STRIDE_Y 1
	#else
		#define LANE_STRIDE_X 1
		#if CONFIG_COMPILE_FP16
			#define LANE_STRIDE_Y 4
		#else
			#define LANE_STRIDE_Y 2
		#endif
	#endif

	#define WAVE_COUNT_X 1
	#define WAVE_COUNT_Y (GROUP_TILE_SIZE / LANE_STRIDE_Y)

	#define LANE_COUNT_X (GROUP_TILE_SIZE / LANE_STRIDE_X)
	#define LANE_COUNT_Y 1

	#if LANE_STRIDE_Y > 1
		// Each wave needs to output top and bottom rows of pixels into LDS;
		#define LDS_SIZE ((2 * LANE_STRIDE_X) * LANE_COUNT_X * WAVE_COUNT_Y)
	#else
		#define LDS_SIZE (GROUP_TILE_SIZE * GROUP_TILE_SIZE)
	#endif

	#ifdef PLATFORM_SPECIFIC_TENSOR_REGISTER_LAYOUT
		#define TENSOR_REGISTER_LAYOUT PLATFORM_SPECIFIC_TENSOR_REGISTER_LAYOUT
	#endif
	
	/** Manually despill to LDS to save more register pressure. */
	#if CONFIG_COMPILE_FP16 && !CONFIG_SCENE_COLOR_ALPHA
		#define CONFIG_LDS_DESPILL 1
		#define CONFIG_CACHE_REFETCH 1
	#endif

#else
	#error Unknown wave size
#endif

#if CONFIG_COMPILE_FP16
	#define LDS_DWORD_COMPONENT_COUNT 2
#else
	#define LDS_DWORD_COMPONENT_COUNT 4
#endif

#ifndef CONFIG_LDS_DESPILL
	#define CONFIG_LDS_DESPILL 0
#endif

/** Whether should minimise register pressure by reloading some data from cache using memory fetches. */
#ifndef CONFIG_CACHE_REFETCH
	#define CONFIG_CACHE_REFETCH 0
#endif

/** Manually despill to LDS on RDNA platforms to save more register pressure. */
#if CONFIG_LDS_DESPILL
	#define LDS_DESPILL_THREAD_COUNT (WAVE_COUNT_X * WAVE_COUNT_Y) * (LANE_COUNT_X * LANE_COUNT_Y)
	#define LDS_DESPILL_OFFSET       (LANE_STRIDE_X * LANE_STRIDE_Y) * GetVectorDwordSize(tsr_half, 4)

	#define LDS_DESPILL_OFFSET_COUNT 2
	
	#define LDS_DESPILL_DWORD_COUNT  LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * LDS_DESPILL_OFFSET_COUNT
#endif

/** Size of the SIMD per lane that also number of input pixel loaded into each individual lanes. */
#define SIMD_SIZE (LANE_STRIDE_X * LANE_STRIDE_Y)


//------------------------------------------------------- INCLUDES

#include "TSRConvolutionNetwork.ush"


//------------------------------------------------------- TYPEDEFS

#define tsr_tensor_float  TLaneVector2D<float, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_float2 TLaneVector2D<float, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_float3 TLaneVector2D<float, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_float4 TLaneVector2D<float, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_floatC TLaneVector2D<float, CONFIG_CHANNEL_COUNT, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_half  TLaneVector2D<tsr_half, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_half2 TLaneVector2D<tsr_half, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_half3 TLaneVector2D<tsr_half, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_half4 TLaneVector2D<tsr_half, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_halfC TLaneVector2D<tsr_half, CONFIG_CHANNEL_COUNT, LANE_STRIDE_X, LANE_STRIDE_Y>

#define tsr_tensor_uint    TLaneVector2D<uint, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_uint2   TLaneVector2D<uint, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_uint3   TLaneVector2D<uint, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_uint4   TLaneVector2D<uint, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_int     TLaneVector2D<int, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_int2    TLaneVector2D<int, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_int3    TLaneVector2D<int, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_int4    TLaneVector2D<int, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_ushort  TLaneVector2D<tsr_ushort, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_ushort2 TLaneVector2D<tsr_ushort, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_ushort3 TLaneVector2D<tsr_ushort, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_ushort4 TLaneVector2D<tsr_ushort, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_short   TLaneVector2D<tsr_short, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_short2  TLaneVector2D<tsr_short, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_short3  TLaneVector2D<tsr_short, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_short4  TLaneVector2D<tsr_short, 4, LANE_STRIDE_X, LANE_STRIDE_Y>

#define tsr_tensor_bool    TLaneVector2D<bool, 1, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_bool2   TLaneVector2D<bool, 2, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_bool3   TLaneVector2D<bool, 3, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_bool4   TLaneVector2D<bool, 4, LANE_STRIDE_X, LANE_STRIDE_Y>
#define tsr_tensor_boolC   TLaneVector2D<bool, CONFIG_CHANNEL_COUNT, LANE_STRIDE_X, LANE_STRIDE_Y>

#ifndef PLATFORM_SPECIFIC_ISOLATE
	#if UE_BINDLESS_ENABLED
		#define PLATFORM_SPECIFIC_ISOLATE
	#else
		#define PLATFORM_SPECIFIC_ISOLATE ISOLATE
	#endif
#endif
