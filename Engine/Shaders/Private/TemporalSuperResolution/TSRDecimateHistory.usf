// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRDepthVelocityAnalysis.ush"
#include "TSRClosestOccluder.ush"
#include "TSRColorSpace.ush"
#include "TSRReprojectionField.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#define CONFIG_KERNEL 0

#define CONFIG_MANUAL_LDS_SPILL 1

/** Whether to compile the moire reprojection. */
#define CONFIG_MOIRE_REPROJECTION (DIM_MOIRE_REPROJECTION)

/** Whether to compile the resurrection reprojection. */
#define CONFIG_RESURRECTION_REPROJECTION (DIM_RESURRECTION_REPROJECTION)

/** Whether to compile the thin geometry coverage reprojection.*/
#define CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION (DIM_THIN_GEOMETRY_COVERAGE_REPROJECTION)

//------------------------------------------------------- PARAMETERS

float4x4 RotationalClipToPrevClip;

Texture2D<uint> DilatedReprojectionVectorTexture;
#if CONFIG_RESURRECTION_REPROJECTION
	Texture2D<float2> ClosestDepthTexture;
#else
	Texture2D<float> ClosestDepthTexture;
#endif
Texture2D<uint> DilateMaskTexture;
Texture2D<uint> DepthErrorTexture;
Texture2DArray<uint> PrevAtomicTextureArray;

Texture2DArray<tsr_half4> PrevHistoryGuide;
Texture2DArray<tsr_half4> PrevHistoryMoire;
#if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
    Texture2DArray<tsr_half> PrevHistoryCoverage;
#endif
float2 PrevGuideInfo_Extent;
float2 PrevGuideInfo_ExtentInverse;
uint2  PrevGuideInfo_ViewportMin;
uint2  PrevGuideInfo_ViewportMax;
float2 PrevGuideInfo_ViewportSize;
float2 PrevGuideInfo_ViewportSizeInverse;
float2 PrevGuideInfo_UVViewportBilinearMin;
float2 PrevGuideInfo_UVViewportBilinearMax;
FScreenTransform InputPixelPosToReprojectScreenPos;
FScreenTransform ScreenPosToPrevHistoryGuideBufferUV;
FScreenTransform ScreenPosToResurrectionGuideBufferUV;
float2 ResurrectionGuideUVViewportBilinearMin;
float2 ResurrectionGuideUVViewportBilinearMax;
float3 HistoryGuideQuantizationError;
float ResurrectionFrameIndex;
float PrevFrameIndex;
float4x4 ClipToResurrectionClip;

RWTexture2DArray<tsr_half4> ReprojectedHistoryGuideOutput;
RWTexture2DArray<tsr_half4> ReprojectedHistoryMoireOutput;
#if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
    RWTexture2DArray<tsr_half> ReprojectedHistoryCoverageOutput;
#endif
RWTexture2DArray<uint> ReprojectionFieldOutput;
RWTexture2D<tsr_half2> DecimateMaskOutput;


//------------------------------------------------------- DEBUG

#define DEBUG_ARRAY_SIZE 8

static float4 Debug[DEBUG_ARRAY_SIZE];


//------------------------------------------------------- LDS

#if CONFIG_MANUAL_LDS_SPILL
	groupshared uint   SharedArray0[TILE_SIZE * TILE_SIZE];
	groupshared float  SharedArray1[TILE_SIZE * TILE_SIZE];
#endif


//------------------------------------------------------- FUNCTIONS

// Correct pre exposure of the guide color.
tsr_halfC CorrectGuideColorExposure(tsr_halfC ReprojectedHistoryGuideColor, const float PreExposureCorrection)
{
	ReprojectedHistoryGuideColor = PreExposureCorrectGCS(ReprojectedHistoryGuideColor, PreExposureCorrection);

	// Removes NaN
	ReprojectedHistoryGuideColor = -min(-ReprojectedHistoryGuideColor, tsr_half(0.0));

	return ReprojectedHistoryGuideColor;
}

#if CONFIG_MOIRE_REPROJECTION
// Sample previous moire history
tsr_half4 ReprojectPrevHistoryMoire(tsr_ushort2 InputPixelPos, float2 TextureUV)
{
	uint2 Random = Rand3DPCG16(int3(InputPixelPos - InputInfo_ViewportMin, View.StateFrameIndexMod8)).xy;
	float2 E = Rand16ToFloat(Random);

	return PrevHistoryMoire.SampleLevel(GlobalPointClampedSampler, float3(TextureUV + (E - 0.5) * PrevGuideInfo_ExtentInverse, 0), 0);
}

// Correct pre exposure of the moire history.
tsr_half4 CorrectMoireExposure(tsr_half4 ReprojectedHistoryMoireColor, const float PreExposureCorrection)
{
	tsr_half PrevFlickeringHistory = ReprojectedHistoryMoireColor[0];

	PrevFlickeringHistory = PreExposureCorrectGCS(PrevFlickeringHistory.xxx, PreExposureCorrection).r;

	// Removes NaN
	PrevFlickeringHistory = -min(-PrevFlickeringHistory, tsr_half(0.0));
	
	return tsr_half4(PrevFlickeringHistory, ReprojectedHistoryMoireColor.yzw);
}
#endif

#if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
tsr_half ReprojectPrevHistoryCoverage(tsr_ushort2 InputPixelPos, float2 TextureUV)
{
	return PrevHistoryCoverage.SampleLevel(GlobalPointClampedSampler, float3(TextureUV, 0), 0);
}
#endif

//ISOLATE
void ReprojectAllPrevTextures(
	uint2 GroupId,
	uint GroupThreadIndex,
	uint EncodedReprojectionVector,
	float DeviceZ,
	out uint HistoryClosestDeviceZSamples0[4],
	out uint HistoryClosestDeviceZSamples1[4],
	out FCatmullRomFetches<tsr_half4> ReprojectedHistoryGuideSamples,
	#if CONFIG_SCENE_COLOR_ALPHA
		out FCatmullRomFetches<tsr_half4> ReprojectedHistoryGuideUncertaintySamples,
	#endif
	#if CONFIG_MOIRE_REPROJECTION
		out tsr_half4 ReprojectedHistoryMoireColor,
	#endif
    #if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
	    out tsr_half ReprojectedHistoryCoverage,
    #endif
	#if CONFIG_RESURRECTION_REPROJECTION
		out tsr_halfC ResurrectedHistoryGuideColor,
		out tsr_half  ResurrectedHistoryGuideUncertainty,
		out bool      bIsResurrectionOffScreen,
	#endif
	out bool bIsOffScreen)
{
	tsr_ushort2 InputPixelPos = (
		Map8x8Tile2x2Lane(GroupThreadIndex) +
		(tsr_ushort2(InputInfo_ViewportMin) + tsr_ushort2(GroupId) * tsr_ushort2(TILE_SIZE, TILE_SIZE)));
			
	{
		float2 ScreenPos = ApplyScreenTransform(float2(InputPixelPos), InputPixelPosToScreenPos);
		float2 PrevScreenPos = ScreenPos - DecodeReprojectionVector(EncodedReprojectionVector);
		LoadPrevAtomicTexturesSamples(
			PrevAtomicTextureArray,
			PrevScreenPos,
			/* out */ HistoryClosestDeviceZSamples0,
			/* out */ HistoryClosestDeviceZSamples1);
	}
			
	{
		float2 OutputScreenPos = ApplyScreenTransform(float2(InputPixelPos), InputPixelPosToReprojectScreenPos);
	
		float2 PrevOutputScreenPos = OutputScreenPos - DecodeReprojectionVector(EncodedReprojectionVector);
		bIsOffScreen = (
			(bCameraCut != 0) ||
			(PrevOutputScreenPos.x < -1.0) ||
			(PrevOutputScreenPos.y < -1.0) ||
			(PrevOutputScreenPos.x > 1.0) ||
			(PrevOutputScreenPos.y > 1.0));
				
		float2 KernelHistoryBufferUV = ApplyScreenTransform(PrevOutputScreenPos, ScreenPosToPrevHistoryGuideBufferUV);

		#if CONFIG_SCENE_COLOR_ALPHA
			ReprojectedHistoryGuideSamples.FetchSamples(
				PrevHistoryGuide,
				KernelHistoryBufferUV,
				PrevFrameIndex * 2.0 + 0.0,
				PrevGuideInfo_Extent,
				PrevGuideInfo_ExtentInverse,
				PrevGuideInfo_UVViewportBilinearMin,
				PrevGuideInfo_UVViewportBilinearMax);
					
			ReprojectedHistoryGuideUncertaintySamples.FetchSamples(
				PrevHistoryGuide,
				KernelHistoryBufferUV,
				PrevFrameIndex * 2.0 + 1.0,
				PrevGuideInfo_Extent,
				PrevGuideInfo_ExtentInverse,
				PrevGuideInfo_UVViewportBilinearMin,
				PrevGuideInfo_UVViewportBilinearMax);
		#else
			ReprojectedHistoryGuideSamples.FetchSamples(
				PrevHistoryGuide,
				KernelHistoryBufferUV,
				PrevFrameIndex,
				PrevGuideInfo_Extent,
				PrevGuideInfo_ExtentInverse,
				PrevGuideInfo_UVViewportBilinearMin,
				PrevGuideInfo_UVViewportBilinearMax);
		#endif

		#if CONFIG_MOIRE_REPROJECTION
		{
			ReprojectedHistoryMoireColor = ReprojectPrevHistoryMoire(InputPixelPos, KernelHistoryBufferUV);
		}
		#endif

        #if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
		{
			ReprojectedHistoryCoverage = ReprojectPrevHistoryCoverage(InputPixelPos, KernelHistoryBufferUV);
		}
        #endif
	}

	#if CONFIG_RESURRECTION_REPROJECTION
	{
		float2 OutputScreenPos = ApplyScreenTransform(float2(InputPixelPos), InputPixelPosToReprojectScreenPos);

		float4 ThisClip = float4(OutputScreenPos, DeviceZ, 1);
		float4 ResurrectionClip = mul(ThisClip, ClipToResurrectionClip); 
		float2 ResurrectionScreen = ResurrectionClip.xy / ResurrectionClip.w;

		bIsResurrectionOffScreen = (
			(ResurrectionScreen.x < -1.0) ||
			(ResurrectionScreen.y < -1.0) ||
			(ResurrectionScreen.x > 1.0) ||
			(ResurrectionScreen.y > 1.0));

		float2 KernelHistoryBufferUV = ApplyScreenTransform(ResurrectionScreen, ScreenPosToResurrectionGuideBufferUV);
		KernelHistoryBufferUV = fastClamp(KernelHistoryBufferUV, ResurrectionGuideUVViewportBilinearMin, ResurrectionGuideUVViewportBilinearMax);

		#if CONFIG_SCENE_COLOR_ALPHA
			ResurrectedHistoryGuideColor       = PrevHistoryGuide.SampleLevel(GlobalBilinearClampedSampler, float3(KernelHistoryBufferUV, ResurrectionFrameIndex * 2.0 + 0.0), 0);
			ResurrectedHistoryGuideUncertainty = PrevHistoryGuide.SampleLevel(GlobalBilinearClampedSampler, float3(KernelHistoryBufferUV, ResurrectionFrameIndex * 2.0 + 1.0), 0).r;
		#else
			tsr_half4 RawGuide                 = PrevHistoryGuide.SampleLevel(GlobalBilinearClampedSampler, float3(KernelHistoryBufferUV, ResurrectionFrameIndex), 0);
			ResurrectedHistoryGuideColor = RawGuide.rgb;
			ResurrectedHistoryGuideUncertainty = RawGuide.a;
		#endif
	}
	#endif
} // ReprojectAllPrevTextures()

//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void DecimateHistoryCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
	{
		Debug[DebugId] = 0.0;
	}

	// Sample current frame informations.
	uint DilatedEncodedReprojectionVector;
	float DeviceZ;
	float PrevDeviceZ;
	float DeviceZError;
	bool bHasPixelAnimation;
#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
	bool bDisableAnimationAA = false;
#endif
	bool bHasReprojectionOffset;
	tsr_half ReprojectionEdge;
	ISOLATE
	{
		uint EncodedReprojectionVectorNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		
		tsr_half PremilinaryReprojectionEdge;

		// Issue texture fetechs
		ISOLATE
		{
			tsr_ushort2 InputPixelPos = (
				Map8x8Tile2x2Lane(GroupThreadIndex) +
				(tsr_ushort2(InputInfo_ViewportMin) + tsr_ushort2(GroupId) * tsr_ushort2(TILE_SIZE, TILE_SIZE)));

			uint EncodedDilateMask = DilateMaskTexture[InputPixelPos];
			uint EncodedDeviceZError = DepthErrorTexture[InputPixelPos];
			#if CONFIG_RESURRECTION_REPROJECTION
				float2 DeviceZAndPrevDeviceZ = ClosestDepthTexture[InputPixelPos];
				#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
					bDisableAnimationAA = DeviceZAndPrevDeviceZ.g < 0;
					DeviceZ = abs(DeviceZAndPrevDeviceZ.g);
				#else
					DeviceZ = DeviceZAndPrevDeviceZ.g;
				#endif
			#else
				float DeviceZAndPrevDeviceZ = ClosestDepthTexture[InputPixelPos];
				DeviceZ = 0.0;
			#endif
			
			bHasReprojectionOffset = (EncodedDilateMask & 0x80u) != 0u;
			PremilinaryReprojectionEdge = tsr_half(tsr_ushort(EncodedDilateMask & 0x7Fu)) * rcp(tsr_half(127.0));
			PrevDeviceZ = abs(DeviceZAndPrevDeviceZ.r);
			bHasPixelAnimation = DeviceZAndPrevDeviceZ.r < 0.0;
			DeviceZError = DecodeDeviceZError(PrevDeviceZ, EncodedDeviceZError);

			FetchReprojectionNeighborhood(DilatedReprojectionVectorTexture, InputPixelPos, /* out */ EncodedReprojectionVectorNeighborhood);
			DilatedEncodedReprojectionVector = AccessNeighborhoodCenter(EncodedReprojectionVectorNeighborhood);
		}

		// LDS spill to save VGPR on the history texture fetches.
		#if CONFIG_MANUAL_LDS_SPILL
		ISOLATE
		{
			SharedArray0[GroupThreadIndex] = DilatedEncodedReprojectionVector;
			SharedArray1[GroupThreadIndex] = PrevDeviceZ;
		}
		#endif
		
		// Detect whether is on velocity edge to clamp the high frequencies
		ISOLATE
		{
			float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
			ComputePixelVelocityNeighborhood(EncodedReprojectionVectorNeighborhood, /* out */ PixelVelocityNeighborhood);

			ReprojectionEdge = ComputeReprojectionEdge(PixelVelocityNeighborhood);
			ReprojectionEdge = select(PremilinaryReprojectionEdge > tsr_half(0.9), PremilinaryReprojectionEdge, ReprojectionEdge);
			ReprojectionEdge = select(bHasReprojectionOffset, ReprojectionEdge, tsr_half(1.0));
		}
	}
	
	// Sample the previous frame information
	uint HistoryClosestDeviceZSamples0[4];
	uint HistoryClosestDeviceZSamples1[4];
	

	FCatmullRomFetches<tsr_half4> ReprojectedHistoryGuideSamples;
	#if CONFIG_SCENE_COLOR_ALPHA
		FCatmullRomFetches<tsr_half4> ReprojectedHistoryGuideUncertaintySamples;
	#endif
	#if CONFIG_MOIRE_REPROJECTION
		tsr_half4 ReprojectedHistoryMoireColor;
	#endif
    #if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
		tsr_half ReprojectedHistoryCoverage;
    #endif
	#if CONFIG_RESURRECTION_REPROJECTION
		tsr_halfC ResurrectedHistoryGuideColor;
		tsr_half  ResurrectedHistoryGuideUncertainty;
		bool bIsResurrectionOffScreen;
	#else
		const bool bIsResurrectionOffScreen = false;
	#endif
	bool bIsOffScreen;

	// Issues all the overlapping texture fetches for previous frame texture using motion vectors
	{
		ReprojectAllPrevTextures(
			GroupId, GroupThreadIndex,
			DilatedEncodedReprojectionVector, DeviceZ,
			/* out */ HistoryClosestDeviceZSamples0,
			/* out */ HistoryClosestDeviceZSamples1,
			/* out */ ReprojectedHistoryGuideSamples,
			#if CONFIG_SCENE_COLOR_ALPHA
				/* out */ ReprojectedHistoryGuideUncertaintySamples,
			#endif
			#if CONFIG_MOIRE_REPROJECTION
				/* out */ ReprojectedHistoryMoireColor,
			#endif
			#if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
				/* out */ ReprojectedHistoryCoverage,
			#endif
			#if CONFIG_RESURRECTION_REPROJECTION
				/* out */ ResurrectedHistoryGuideColor,
				/* out */ ResurrectedHistoryGuideUncertainty,
				/* out */ bIsResurrectionOffScreen,
			#endif
			/* out */ bIsOffScreen);
	}
	
	// Process the history samples
	tsr_halfC ReprojectedHistoryGuideColor;
	tsr_half  ReprojectedHistoryGuideUncertainty;
	bool bIsParallaxDisocclusion;
	bool bReprojectionHollFill;
	//ISOLATE
	{
		#if CONFIG_SCENE_COLOR_ALPHA
		{
			ReprojectedHistoryGuideColor = saturate(ReprojectedHistoryGuideSamples.AccumulateSamples());
			ReprojectedHistoryGuideUncertainty = ReprojectedHistoryGuideUncertaintySamples.AccumulateSamples().r;
		}
		#else
		{
			tsr_half4 RawGuide = ReprojectedHistoryGuideSamples.AccumulateSamples();
			ReprojectedHistoryGuideColor = RawGuide.rgb;
			ReprojectedHistoryGuideUncertainty = RawGuide.a;
		}
		#endif
		
		ReprojectedHistoryGuideColor = CorrectGuideColorExposure(ReprojectedHistoryGuideColor, HistoryPreExposureCorrection);

		#if CONFIG_MOIRE_REPROJECTION
		{
			ReprojectedHistoryMoireColor = CorrectMoireExposure(ReprojectedHistoryMoireColor, HistoryPreExposureCorrection);
		}
		#endif

		#if CONFIG_RESURRECTION_REPROJECTION
		{
			ResurrectedHistoryGuideColor = CorrectGuideColorExposure(ResurrectedHistoryGuideColor, ResurrectionPreExposureCorrection);
		}
		#endif

		// LDS despill
		#if CONFIG_MANUAL_LDS_SPILL
		{
			DilatedEncodedReprojectionVector = SharedArray0[GroupThreadIndex];
			PrevDeviceZ = SharedArray1[GroupThreadIndex];
		}
		#endif
		
		tsr_ushort2 InputPixelPos = (
			Map8x8Tile2x2Lane(GroupThreadIndex) +
			(tsr_ushort2(InputInfo_ViewportMin) + tsr_ushort2(GroupId) * tsr_ushort2(TILE_SIZE, TILE_SIZE)));

		float2 ScreenPos = ApplyScreenTransform(float2(InputPixelPos), InputPixelPosToScreenPos);
		float2 ScreenVelocity = DecodeReprojectionVector(DilatedEncodedReprojectionVector);
		
		// Compute the parralax rejection mask
		float2 HoleFillingPixelVelocity;
		bool bCanHoleFill;
		ProcessPrevAtomicTexturesSamples(
			HistoryClosestDeviceZSamples0,
			HistoryClosestDeviceZSamples1,
			ScreenPos,
			ScreenVelocity,
			PrevDeviceZ,
			DeviceZError,
			bIsOffScreen,
			/* out */ bIsParallaxDisocclusion,
			/* out */ HoleFillingPixelVelocity,
			/* out */ bCanHoleFill);

		// LDS despill
		#if CONFIG_MANUAL_LDS_SPILL
		{
			DilatedEncodedReprojectionVector = SharedArray0[GroupThreadIndex];
		}
		#endif
	
		bReprojectionHollFill = bCanHoleFill && bIsParallaxDisocclusion;

		// If the hole filling velocity has better result that the neighborhood, use that.
		BRANCH
		if (bReprojectionHollFill)
		{
			float2 HoleFillingReprojectionVector = HoleFillingPixelVelocity * InputPixelVelocityToScreenVelocity;
			DilatedEncodedReprojectionVector = EncodeReprojectionVector(HoleFillingReprojectionVector);
		}
	}
	
	// Builds the decimate bitmask
	tsr_ushort DecimateBitMask = 
		select(bIsOffScreen,             tsr_ushort(0x01), tsr_ushort(0x0)) |
		select(bIsParallaxDisocclusion,  tsr_ushort(0x02), tsr_ushort(0x0)) |
		select(bHasPixelAnimation,       tsr_ushort(0x04), tsr_ushort(0x0)) |
		select(bReprojectionHollFill,    tsr_ushort(0x08), tsr_ushort(0x0)) |
#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS 
		select(bDisableAnimationAA,      tsr_ushort(0x20), tsr_ushort(0x0)) |
#endif
		select(bIsResurrectionOffScreen, tsr_ushort(0x10), tsr_ushort(0x0));

	// Output full res
	ISOLATE
	{
		tsr_ushort2 InputPixelPos = (
			Map8x8Tile2x2Lane(GroupThreadIndex) +
			(tsr_ushort2(InputInfo_ViewportMin) + tsr_ushort2(GroupId) * tsr_ushort2(TILE_SIZE, TILE_SIZE)));
			
		#if CONFIG_ENABLE_STOCASTIC_QUANTIZATION && 1
		{
			uint Random = Rand3DPCG16(int3(InputPixelPos - InputInfo_ViewportMin, View.StateFrameIndexMod8)).x;
			tsr_half E = tsr_half(Rand16ToFloat(Random));

			ReprojectedHistoryGuideColor = QuantizeForUNormRenderTarget(ReprojectedHistoryGuideColor, E, HistoryGuideQuantizationError);
		}
		#endif
		
		tsr_short2 OutputPixelPos = InvalidateOutputPixelPos(InputPixelPos, InputInfo_ViewportMax);

		{
			#if CONFIG_SCENE_COLOR_ALPHA
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 0)] = ReprojectedHistoryGuideColor;
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 1)] = tsr_half4(ReprojectedHistoryGuideUncertainty, 0, 0, 0);
			#else
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 0)] = tsr_half4(ReprojectedHistoryGuideColor.xyz, ReprojectedHistoryGuideUncertainty);
			#endif
			#if CONFIG_MOIRE_REPROJECTION
				ReprojectedHistoryMoireOutput[tsr_short3(OutputPixelPos, 0)] = ReprojectedHistoryMoireColor;
			#endif
			#if CONFIG_THIN_GEOMETRY_COVERAGE_REPROJECTION
				ReprojectedHistoryCoverageOutput[tsr_short3(OutputPixelPos, 0)] = ReprojectedHistoryCoverage;
			#endif
		}

		#if CONFIG_RESURRECTION_REPROJECTION
		{
			#if CONFIG_SCENE_COLOR_ALPHA
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 2)] = ResurrectedHistoryGuideColor;
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 3)] = tsr_half4(ResurrectedHistoryGuideUncertainty, 0, 0, 0);
			#else
				ReprojectedHistoryGuideOutput[tsr_short3(OutputPixelPos, 1)] = tsr_half4(ResurrectedHistoryGuideColor.xyz, ResurrectedHistoryGuideUncertainty);
			#endif
		}
		#endif
		
		DecimateMaskOutput[OutputPixelPos] = tsr_half2(tsr_half(DecimateBitMask) * rcp(tsr_half(255.0)), ReprojectionEdge);

		// Selectively overwrite the reprojection field
		{
			bool bWriteReprojectionVector   = bReprojectionHollFill || true;
			bool bWriteReprojectionJacobian = bReprojectionHollFill;

			// Disable the reprojection's jacobian since this is to extrude disoccluded area we don't know how they look anyway.
			const uint EncodedReprojectionJacobian = EncodeReprojectionJacobian(tsr_half2x2(tsr_half(0.0).xxxx));

			ReprojectionFieldOutput[tsr_short3(select(bWriteReprojectionVector,   OutputPixelPos.x, -tsr_short(1)), OutputPixelPos.y, kReprojectionVectorOutputIndex)]   = DilatedEncodedReprojectionVector;
			ReprojectionFieldOutput[tsr_short3(select(bWriteReprojectionJacobian, OutputPixelPos.x, -tsr_short(1)), OutputPixelPos.y, kReprojectionJacobianOutputIndex)] = EncodedReprojectionJacobian;
		}

		#if DEBUG_OUTPUT
		for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
		{
			DebugOutput[tsr_short3(OutputPixelPos, DebugId)] = Debug[DebugId];
		}
		#endif
	}
}
