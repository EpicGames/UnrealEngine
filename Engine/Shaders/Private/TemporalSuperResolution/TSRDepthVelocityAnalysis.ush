// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRKernels.ush"
#include "TSRReprojectionField.ush"


//------------------------------------------------------- CONFIG

/** Whether to use the velocity from SceneVelocityTexture. */
#define CONFIG_USE_SCENE_VELOCITY 1

/** Whether the dilation should be only on depth edges, improving texture reprojection accuracy on flat surfaces. */
#define CONFIG_DEPTH_EDGE_DETECTION 1


//------------------------------------------------------- DEFINITIONS

#define MAX_DEVICE_Z_ERROR 64.0

#define CONFIG_VELOCITY_SAMPLES 9
#define CONFIG_VELOCITY_SAMPLE_PAIRS ((CONFIG_VELOCITY_SAMPLES - 1) / 2)


//------------------------------------------------------- DEPTH & VELOCITY UTILIES

CALL_SITE_DEBUGLOC
float3 ComputeStaticVelocity(float2 ScreenPos, float DeviceZ)
{
	float3 PosN = float3(ScreenPos, DeviceZ);

	float4 ThisClip = float4(PosN, 1);
	float4 PrevClip = mul( ThisClip, View.ClipToPrevClip );
	float3 PrevScreen = PrevClip.xyz / PrevClip.w;
	return PosN - PrevScreen;
}

/** Compute the devize Z error due to the size of a pixel in world space at that given depth. */
CALL_SITE_DEBUGLOC
float ComputePixelDeviceZError(float DeviceZ)
{
	float Depth = ConvertFromDeviceZ(DeviceZ);
	float ErrorCorrectedDeviceZ = ConvertToDeviceZ(GetDepthPixelRadiusForProjectionType(Depth) * 2.0f + Depth);
	float DeviceZError = abs(ErrorCorrectedDeviceZ - DeviceZ);
	return DeviceZError;
}

/** Encode the devize Z error relatively to the closest depth to squeeze to 8bit per pixel */
CALL_SITE_DEBUGLOC
uint EncodeDeviceZError(float PrevClosestDeviceZ, float ParallaxDepthError)
{
	float PixelDeviceZError = float(MAX_DEVICE_Z_ERROR) * ComputePixelDeviceZError(PrevClosestDeviceZ);
	float RelativeZError = ParallaxDepthError / PixelDeviceZError;
	
	uint EncodedDeviceZError = uint(ceil(saturate(RelativeZError) * 255.0));
	return EncodedDeviceZError;
}

/** Inverse of EncodeDeviceZError() */
CALL_SITE_DEBUGLOC
float DecodeDeviceZError(float PrevClosestDeviceZ, uint EncodedDeviceZError)
{
	float PixelDeviceZError = ComputePixelDeviceZError(PrevClosestDeviceZ);
	float RelativeZError = float(EncodedDeviceZError) * (float(MAX_DEVICE_Z_ERROR) / 255.0);
	float ParallaxDepthError = PixelDeviceZError * RelativeZError;
	return ParallaxDepthError;
}


//------------------------------------------------------- DEPTH & VELOCITY FETCHES & PROCESSING

/* Loads all the 3x3's depth. */
CALL_SITE_DEBUGLOC
void FetchDepth3x3(
	Texture2D<float> SceneDepthTexture,
	tsr_short2 InputPixelPos,
	out float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
	for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
	{
		tsr_short2 SampleInputPixelPos = GetNeighborhoodPixelPos(i, InputPixelPos);
		
		DevizeZNeighborhood[i] = SceneDepthTexture[SampleInputPixelPos];
	}
} // FetchDepthVelocity3x3()

/* Loads all the 3x3's depth and velocities. */
CALL_SITE_DEBUGLOC
void FetchDepthVelocity3x3(
	Texture2D<float> SceneDepthTexture,
	Texture2D<float4> SceneVelocityTexture,
	tsr_short2 InputPixelPos,
	out float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out float4 VelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
	for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
	{
		tsr_short2 SampleInputPixelPos = GetNeighborhoodPixelPos(i, InputPixelPos);
		
		DevizeZNeighborhood[i] = SceneDepthTexture[SampleInputPixelPos];
		VelocityNeighborhood[i] = SceneVelocityTexture[SampleInputPixelPos];
	}
} // FetchDepthVelocity3x3()

CALL_SITE_DEBUGLOC
void FetchAndComputeScreenVelocity(
	Texture2D<float4> SceneVelocityTexture,
	tsr_short2 InputPixelPos,
	float ClosestDeviceZ,
	tsr_short2 ReprojectionOffset,
	out float3 ScreenVelocity,
	out bool bHasPixelAnimation,
	out bool bDisableAnimationAA)
{
	float4 EncodedVelocity = SceneVelocityTexture[InputPixelPos + ReprojectionOffset];

	#if VELOCITY_ENCODE_HAS_PIXEL_ANIMATION
		bHasPixelAnimation = DecodeHasPixelAnimationFromVelocityTexture(EncodedVelocity);
	#else
		bHasPixelAnimation = false;
	#endif

	#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
		bDisableAnimationAA = DecodeDisableAnimationAAFromVelocityTexture(EncodedVelocity);
	#else
		bDisableAnimationAA = false;
	#endif
	
	#if CONFIG_USE_SCENE_VELOCITY
	BRANCH
	if (EncodedVelocity.x > 0.0)
	{
		ScreenVelocity = DecodeVelocityFromTexture(EncodedVelocity);
	}
	else
	#endif
	{
		float2 ScreenPos = ApplyScreenTransform(float2(int2(InputPixelPos)), InputPixelPosToScreenPos);
		ScreenVelocity = ComputeStaticVelocity(ScreenPos, ClosestDeviceZ);
	}
} // FetchAndComputeScreenVelocity()

/** Convert the fetches velocities to pixel velocities. */
void ComputePixelVelocityNeighborhood(
	tsr_short2 InputPixelPos,
	float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	float4 VelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out float  DepthVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out bool   bIsDrawingVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	bool bHasAnyStaticPixels = false;
	bool bHasAnyDynamicPixels = false;
	{
		UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
		for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
		{
			#if CONFIG_USE_SCENE_VELOCITY
				bool bIsDrawingVelocity = VelocityNeighborhood[i].x > 0.0;
			#else
				const bool bIsDrawingVelocity = false;
			#endif
			bIsDrawingVelocityNeighborhood[i] = bIsDrawingVelocity;
			bHasAnyStaticPixels  |= !bIsDrawingVelocity;
			bHasAnyDynamicPixels |= bIsDrawingVelocity;
		}

		#if COMPILER_SUPPORTS_WAVE_VOTE
			bHasAnyStaticPixels  = WaveActiveAnyTrue(bHasAnyStaticPixels);
			bHasAnyDynamicPixels = WaveActiveAnyTrue(bHasAnyDynamicPixels);
		#else
			bHasAnyStaticPixels  = true;
			bHasAnyDynamicPixels = true;
		#endif
	}
	
	#if CONFIG_USE_SCENE_VELOCITY
	BRANCH
	if (bHasAnyStaticPixels && bHasAnyDynamicPixels)
	{
		UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
		for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
		{
			tsr_short2 SampleInputPixelPos = GetNeighborhoodPixelPos(i, InputPixelPos);
			float2 ScreenPos = ApplyScreenTransform(float2(int2(SampleInputPixelPos)), InputPixelPosToScreenPos);

			float3 ScreenVelocity = ComputeStaticVelocity(ScreenPos, DevizeZNeighborhood[i]);
			FLATTEN
			if (VelocityNeighborhood[i].x > 0.0)
			{
				ScreenVelocity = DecodeVelocityFromTexture(VelocityNeighborhood[i]);
			}

			PixelVelocityNeighborhood[i] = ScreenVelocity.xy * ScreenVelocityToInputPixelVelocity;
			DepthVelocityNeighborhood[i] = ScreenVelocity.z;
		}
	}
	else if (bHasAnyDynamicPixels) // && !bHasAnyStaticPixels)
	{
		UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
		for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
		{
			float3 ScreenVelocity = DecodeVelocityFromTexture(VelocityNeighborhood[i]);

			PixelVelocityNeighborhood[i] = ScreenVelocity.xy * ScreenVelocityToInputPixelVelocity;
			DepthVelocityNeighborhood[i] = ScreenVelocity.z;
		}
	}
	else // if (bHasAnyStaticPixels && !bHasAnyDynamicPixels
	#endif // !CONFIG_USE_SCENE_VELOCITY
	{
		UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
		for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
		{
			tsr_short2 SampleInputPixelPos = GetNeighborhoodPixelPos(i, InputPixelPos);
			float2 ScreenPos = ApplyScreenTransform(float2(int2(SampleInputPixelPos)), InputPixelPosToScreenPos);

			float3 ScreenVelocity = ComputeStaticVelocity(ScreenPos, DevizeZNeighborhood[i]);

			PixelVelocityNeighborhood[i] = ScreenVelocity.xy * ScreenVelocityToInputPixelVelocity;
			DepthVelocityNeighborhood[i] = ScreenVelocity.z;
		}
	}
} // ComputePixelVelocityNeighborhood()


//------------------------------------------------------- REPROJECTION FIELD FETCH & PROCESSING

CALL_SITE_DEBUGLOC
void FetchReprojectionNeighborhood(
	Texture2D<uint> ReprojectionVectorTexture,
	tsr_short2 InputPixelPos,
	out uint EncodedReprojectionVectorNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	tsr_short2 LaneOffsetSign = GetLaneOffsetSign();

	UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
	for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
	{
		tsr_short2 SampleInputPixelPos = GetNeighborhoodPixelPos(i, InputPixelPos);
		EncodedReprojectionVectorNeighborhood[i] = ReprojectionVectorTexture[SampleInputPixelPos];
	}
} // FetchReprojectionNeighborhood()

CALL_SITE_DEBUGLOC
void ComputePixelVelocityNeighborhood(
	uint EncodedReprojectionVectorNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	UNROLL_N(CONFIG_BUTTERFLY_SAMPLES)
	for (uint i = 0; i < CONFIG_BUTTERFLY_SAMPLES; i++)
	{
		PixelVelocityNeighborhood[i] = DecodeReprojectionVector(EncodedReprojectionVectorNeighborhood[i]) * ScreenVelocityToInputPixelVelocity;
	}
} // ComputePixelVelocityNeighborhood()


//------------------------------------------------------- DEPTH ANALYSIS

CALL_SITE_DEBUGLOC
uint PackDepthAndOffset(const int2 Offset, float DeviceZ)
{
	return ((asuint(DeviceZ) << 2) & ~0xF) | (uint(1 + Offset.x) << 0) | (uint(1 + Offset.y) << 2);
}

CALL_SITE_DEBUGLOC
void UnpackDepthAndOffset(uint Packed, out tsr_short2 Offset, out float DeviceZ)
{
	DeviceZ = asfloat((Packed & ~0xF) >> 2);
	Offset.x = tsr_short((Packed >> 0u) & 0x3u) - tsr_short(1);
	Offset.y = tsr_short((Packed >> 2u) & 0x3u) - tsr_short(1);
}

/** Find the closest DevizeZ and its corresponding offset */
void FindClosestDepthOffset(
	float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out float ClosestDeviceZ,
	out tsr_short2 ReprojectionOffset)
{
	float DeviceZ = AccessNeighborhoodCenter(DevizeZNeighborhood);
	float DeviceZError = ComputePixelDeviceZError(DeviceZ);
	uint PackedDepthOffset = PackDepthAndOffset(/* Offset = */ int2(0, 0), DeviceZ);
	
	#if 0
		UNROLL_N(4)
		for (uint i = 0; i < 4; i++)
	#elif 1
		UNROLL_N(2)
		for (uint i = 0; i < 4; i+= 2)
	#else
		UNROLL_N(2)
		for (uint i = 1; i < 4; i+= 2)
	#endif
	{
		const int2 PairOffset = kPairOffsets[i];

		float SampleDeviceZ0 = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, +PairOffset);
		float SampleDeviceZ1 = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, -PairOffset);

		#if CONFIG_DEPTH_EDGE_DETECTION
		{
			float DepthDiff = abs(SampleDeviceZ1 - SampleDeviceZ0);
			float DepthVariation = (SampleDeviceZ0 + SampleDeviceZ1) * 0.5 - DeviceZ;

			FLATTEN
			if (DepthVariation > max(DepthDiff * 0.25, DeviceZError))
			{
				PackedDepthOffset = max(PackedDepthOffset, PackDepthAndOffset(+PairOffset, SampleDeviceZ0));
				PackedDepthOffset = max(PackedDepthOffset, PackDepthAndOffset(-PairOffset, SampleDeviceZ1));
			}
		}
		#else
		{
			PackedDepthOffset = max(PackedDepthOffset, PackDepthAndOffset(+PairOffset, SampleDeviceZ0));
			PackedDepthOffset = max(PackedDepthOffset, PackDepthAndOffset(-PairOffset, SampleDeviceZ1));
		}
		#endif
	}

	UnpackDepthAndOffset(PackedDepthOffset, /* out */ ReprojectionOffset, /* out */ ClosestDeviceZ);
	
	#if CONFIG_BUTTERFLY_KERNEL
		tsr_short2 LaneOffsetSign = GetLaneOffsetSign();
		ReprojectionOffset *= LaneOffsetSign;
	#endif
} // FindClosestDepthOffset()

/** Compute the velocity jacobian along a two orthogonal axes, taking a common closest depth. */
bool ComputeDepthBilaterals(
	const int2 DirectionE,
	const int2 DirectionS,
	bool bCompute,
	float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	inout float DepthError,
	out tsr_half BilateralWeights[4])
{
	float DevizeZC = AccessNeighborhoodCenter(DevizeZNeighborhood);
	float DevizeZE = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, +DirectionE);
	float DevizeZW = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, -DirectionE);
	float DevizeZS = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, +DirectionS);
	float DevizeZN = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, -DirectionS);
	
	tsr_half WeightE;
	tsr_half WeightW;
	tsr_half WeightS;
	tsr_half WeightN;
		
	bool bSuccess;
	BRANCH
	if (bCompute)
	{
		const float WeightOffset = 1.5;
		const float PixelDistance = length(float2(DirectionE));

		float VariationEW = abs(DevizeZC - (DevizeZE + DevizeZW) * 0.5);
		float VariationSN = abs(DevizeZC - (DevizeZS + DevizeZN) * 0.5);

		float DepthDiffE = abs(DevizeZE - DevizeZC);
		float DepthDiffW = abs(DevizeZW - DevizeZC);
		float DepthDiffS = abs(DevizeZS - DevizeZC);
		float DepthDiffN = abs(DevizeZN - DevizeZC);
	
		float DepthDiffEW = abs(DevizeZE - DevizeZW);
		float DepthDiffSN = abs(DevizeZS - DevizeZN);

		float MinDepthDerivativeE = min3(DepthDiffEW, DepthDiffE, DepthDiffW);
		float MinDepthDerivativeS = min3(DepthDiffSN, DepthDiffS, DepthDiffN);
		
		float PixelDeviceZError = ComputePixelDeviceZError(DevizeZC) * PixelDistance;

		float FinalDepthDerivativeE = min(max(MinDepthDerivativeE, PixelDeviceZError), max(MinDepthDerivativeS, 8.0 * PixelDeviceZError));
		float FinalDepthDerivativeS = min(max(MinDepthDerivativeS, PixelDeviceZError), max(MinDepthDerivativeE, 8.0 * PixelDeviceZError));

		WeightE = tsr_half(saturate(WeightOffset - min(DepthDiffE, VariationEW) / FinalDepthDerivativeE));
		WeightW = tsr_half(saturate(WeightOffset - min(DepthDiffW, VariationEW) / FinalDepthDerivativeE));
		WeightS = tsr_half(saturate(WeightOffset - min(DepthDiffS, VariationSN) / FinalDepthDerivativeS));
		WeightN = tsr_half(saturate(WeightOffset - min(DepthDiffN, VariationSN) / FinalDepthDerivativeS));

	
		tsr_half TotalWeightEW = WeightE + WeightW;
		tsr_half TotalWeightSN = WeightS + WeightN;
	
		bool bSuccessEW = TotalWeightEW > tsr_half(0.0);
		bool bSuccessSN = TotalWeightSN > tsr_half(0.0);

		tsr_half InvTotalWeightEW = select(bSuccessEW, rcp(TotalWeightEW), tsr_half(0.0));
		tsr_half InvTotalWeightSN = select(bSuccessSN, rcp(TotalWeightSN), tsr_half(0.0));

		WeightE *= InvTotalWeightEW;
		WeightW *= InvTotalWeightEW;
		WeightS *= InvTotalWeightSN;
		WeightN *= InvTotalWeightSN;
		bSuccess = bSuccessEW || bSuccessSN;
		
		DepthError = max(DepthDiffE * WeightE + DepthDiffW * WeightW, DepthDiffS * WeightS + DepthDiffN * WeightN);
	}
	else
	{
		WeightE = tsr_half(1.0);
		WeightW = tsr_half(1.0);
		WeightS = tsr_half(1.0);
		WeightN = tsr_half(1.0);
		bSuccess = false;
	}

	BilateralWeights[0] = WeightE;
	BilateralWeights[1] = WeightW;
	BilateralWeights[2] = WeightS;
	BilateralWeights[3] = WeightN;

	return bSuccess;
} // ComputeDepthBilaterals()

/** Compute the depth error on horizontal and vertical. */
float ComputeDepthError(float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	float DepthError;
	tsr_half BilateralWeights[4];

	ComputeDepthBilaterals(
		kOffsetE, kOffsetS,
		/* bCompute = */ true,
		DevizeZNeighborhood,
		/* inout */ DepthError,
		/* out */ BilateralWeights);

	return DepthError;
}


//------------------------------------------------------- VELOCITY ANALYSIS

/** Computes the velocity's jacobian from the depth bilateral weighs. */
CALL_SITE_DEBUGLOC
void ComputeVelocityJacobian(
	const int2 DirectionE,
	const int2 DirectionS,
	bool bCompute,
	float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	tsr_half BilateralWeights[4],
	inout tsr_half2x2 ReprojectionJacobian)
{
	
	float2 PixelVelocityC = AccessNeighborhoodCenter(PixelVelocityNeighborhood);
	float2 PixelVelocityE = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, +DirectionE);
	float2 PixelVelocityW = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, -DirectionE);
	float2 PixelVelocityS = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, +DirectionS);
	float2 PixelVelocityN = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, -DirectionS);
	
	tsr_half WeightE = BilateralWeights[0];
	tsr_half WeightW = BilateralWeights[1];
	tsr_half WeightS = BilateralWeights[2];
	tsr_half WeightN = BilateralWeights[3];

	BRANCH
	if (bCompute)
	{
		tsr_half2 VelocityDeltaE = tsr_half2(PixelVelocityE - PixelVelocityC);
		tsr_half2 VelocityDeltaW = tsr_half2(PixelVelocityC - PixelVelocityW);
		tsr_half2 VelocityDeltaS = tsr_half2(PixelVelocityS - PixelVelocityC);
		tsr_half2 VelocityDeltaN = tsr_half2(PixelVelocityC - PixelVelocityN);

		ReprojectionJacobian[0] = VelocityDeltaE * WeightE + VelocityDeltaW * WeightW;
		ReprojectionJacobian[1] = VelocityDeltaS * WeightS + VelocityDeltaN * WeightN;
	}
} // ComputeVelocityJacobian()

/** Find the closest DevizeZ and its corresponding offset */
void ComputeReprojectionJacobian(
	float  DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES],
	out tsr_half2x2 ReprojectionJacobian,
	out float DepthError)
{
	float DeviceZ = AccessNeighborhoodCenter(DevizeZNeighborhood);
	
	ReprojectionJacobian = tsr_half(0.0);
	DepthError = 0.0;

	// Horizontal & vertical cardinals first as the pixels are closer than on diagonals
	bool bSuccess;
	{
		const bool bCompute = true;
		const int2 DirectionE = kOffsetE;
		const int2 DirectionS = kOffsetS;

		tsr_half BilateralWeights[4];
		bSuccess = ComputeDepthBilaterals(
			DirectionE, DirectionS,
			bCompute,
			DevizeZNeighborhood,
			/* inout */ DepthError,
			/* out */ BilateralWeights);
		
		ComputeVelocityJacobian(
			DirectionE, DirectionS,
			bCompute,
			PixelVelocityNeighborhood,
			BilateralWeights,
			/* inout */ ReprojectionJacobian);
	}
	
	// Compute diagonal jacobian if the horizontal/vertical failed
#if CONFIG_BUTTERFLY_KERNEL && COMPILER_SUPPORTS_WAVE_VOTE
	if (WaveActiveAnyTrue(!bSuccess))
#endif
	{
		const bool bCompute = !bSuccess;
		const int2 DirectionE = kOffsetSE;
		const int2 DirectionS = kOffsetSW;
		
		tsr_half BilateralWeights[4];
		ComputeDepthBilaterals(
			DirectionE, DirectionS,
			bCompute,
			DevizeZNeighborhood,
			/* inout */ DepthError,
			/* out */ BilateralWeights);
		
		ComputeVelocityJacobian(
			DirectionE, DirectionS,
			bCompute,
			PixelVelocityNeighborhood,
			BilateralWeights,
			/* inout */ ReprojectionJacobian);
			
		// Rotate the diagonal jacobian back to horizontal/vertical
		if (bCompute)
		{
			tsr_half2 DifferentialSE = ReprojectionJacobian[0];
			tsr_half2 DifferentialSW = ReprojectionJacobian[1];
			ReprojectionJacobian[0] = DifferentialSE * tsr_half(+0.5) + DifferentialSW * tsr_half(-0.5);
			ReprojectionJacobian[1] = DifferentialSE * tsr_half(+0.5) + DifferentialSW * tsr_half(+0.5);
		}
	}
	
	// Flips the signs due to the butterfly kernel.
	#if CONFIG_BUTTERFLY_KERNEL && 1
		tsr_half2 LaneOffsetSign = tsr_half2(GetLaneOffsetSign());
		ReprojectionJacobian[0] *= LaneOffsetSign.x;
		ReprojectionJacobian[1] *= LaneOffsetSign.y;
	#endif
} // ComputeReprojectionJacobian()

/** Compute the reprojection edge from non-dilated to dilated screen velocity. */
CALL_SITE_DEBUGLOC
tsr_half ComputeReprojectionEdge(
	float2 ScreenVelocity,
	float2 DilatedScreenVelocity)
{
	tsr_half2 NeighborDeltaPixelVelocity = tsr_half2((DilatedScreenVelocity - ScreenVelocity) * ScreenVelocityToInputPixelVelocity);
	tsr_half ReprojectionEdge = saturate(tsr_half(1.1) - dot(abs(NeighborDeltaPixelVelocity), tsr_half(1.1).xx));
	return ReprojectionEdge;
}

/** Compute the reprojection edge from the pixel velocity neighborhood. */
tsr_half ComputeReprojectionEdge(
	float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	tsr_half ReprojectionEdge = tsr_half(0.0);

	// Sample
	float2 ScreenPixelVelocity = AccessNeighborhoodCenter(PixelVelocityNeighborhood);
	tsr_short2 LaneOffsetSign = GetLaneOffsetSign();

	UNROLL_N(CONFIG_VELOCITY_SAMPLE_PAIRS)
	for (uint i = 0; i < CONFIG_VELOCITY_SAMPLE_PAIRS; i++)
	ISOLATE
	{
		const uint NeightbordId0 = i + 1;
		const uint NeightbordId1 = i + 1 + CONFIG_VELOCITY_SAMPLE_PAIRS;
			
		#if CONFIG_VELOCITY_SAMPLES == 9
			const int2 Offset0 = kOffsets3x3[kSquareIndexes3x3[NeightbordId0]];
			const int2 Offset1 = kOffsets3x3[kSquareIndexes3x3[NeightbordId1]];
		#elif CONFIG_VELOCITY_SAMPLES == 5
			const int2 Offset0 = kOffsets3x3[kPlusIndexes3x3[NeightbordId0]];
			const int2 Offset1 = kOffsets3x3[kPlusIndexes3x3[NeightbordId1]];
		#endif

		float2 NeighborPixelVelocity0 = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, Offset0);
		float2 NeighborPixelVelocity1 = AccessNeighborhoodStaticOffset(PixelVelocityNeighborhood, Offset1);

		tsr_half2 NeighborDeltaPixelVelocity0 = tsr_half2(ScreenPixelVelocity - NeighborPixelVelocity0);
		tsr_half2 NeighborDeltaPixelVelocity1 = tsr_half2(ScreenPixelVelocity - NeighborPixelVelocity1);
			
		// Reject high frequency on reprojection edge discontinuity.
		{
			tsr_half IsOverlapingEdge0 = tsr_half(1.0) - dot(saturate(abs(NeighborDeltaPixelVelocity0 * tsr_half2(tsr_short2(Offset0) * LaneOffsetSign))), tsr_half(1.0).xx); // * saturate(ScreenPixelVelocityLength - NeighborPixelVelocityLength0);
			tsr_half IsOverlapingEdge1 = tsr_half(1.0) - dot(saturate(abs(NeighborDeltaPixelVelocity1 * tsr_half2(tsr_short2(Offset1) * LaneOffsetSign))), tsr_half(1.0).xx); // * saturate(ScreenPixelVelocityLength - NeighborPixelVelocityLength1);
				
			tsr_half IsOverlapingEdge = min(IsOverlapingEdge0, IsOverlapingEdge1);

			if (i == 0)
				ReprojectionEdge = IsOverlapingEdge;
			else
				ReprojectionEdge = min(ReprojectionEdge, IsOverlapingEdge);
		}
	} // for (uint i = 0; i < CONFIG_VELOCITY_SAMPLE_PAIRS; i++)

	// Remove noise from high frequency rejections.
	ReprojectionEdge = saturate(ReprojectionEdge * tsr_half(1.1));
	return ReprojectionEdge;
} // ComputeReprojectionEdge()
