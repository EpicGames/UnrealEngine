// Copyright Epic Games, Inc. All Rights Reserved.

#define CONFIG_INTENSITY_LINE_DETECTION (DIM_INTENSITY_LINE_DETECTION)
#define CONFIG_TRANSLUCENCY_COVERAGE_BIT (DIM_REJECT_TRANSLUCENCY_DETECTION)

//------------------------------------------------------- INCLUDES
#include "TSRColorSpace.ush"
#include "TSRDepthVelocityAnalysis.ush"
#include "TSRConvolutionNetworkPass.ush"
#include "TSRReprojectionField.ush"
#include "TSRThinGeometryCommon.ush"
//------------------------------------------------------- CONFIG

#define DEBUG_ARRAY_SIZE 8

#define DEBUG_DISABLE_PADDING 0

#define DEBUG_DIGESTABLE 0

#define EDGE_HORIZONTAL_VERTICAL 1
#define EDGE_HORIZONTAL_VERTICAL_DIAGNAL 2

#define EDGE_TYPE EDGE_HORIZONTAL_VERTICAL_DIAGNAL

#define CONFIG_VARYING_RANGE_DETECTION (DIM_VARYING_RANGE_DETECTION)

#if CONFIG_INTENSITY_LINE_DETECTION
#define TLineEdgeType tsr_tensor_half2
#else
#define TLineEdgeType tsr_tensor_half
#endif
//------------------------------------------------------- CONSTANTS

#if CONFIG_LDS_DESPILL
	static const uint kDespillInput = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 0;
	static const uint kDespillHistory = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 1;
#endif

//------------------------------------------------------- PARAMETERS

int TileOverscan;
int ThinGeometryTextureIndex;
float ErrorMultiplier;
float MaxRelaxationWeight;
float MinKeepLineContrast;
float HighContrastLineFadeRate;
float HighContrastLineFadeRateInsideRegion;
float HighContrastLineWeight;

Texture2D<float> SceneDepthTexture;
Texture2D<tsr_half> CurrentCoverageTexture;
#if CONFIG_INTENSITY_LINE_DETECTION
Texture2D<tsr_half> FlickeringLumaTexture;
#endif
Texture2D<tsr_half> ReprojectedHistoryCoverageTexture;
Texture2D<tsr_half2> DecimateMaskTexture;

RWTexture2DArray<tsr_half> HistoryCoverageOutput;
RWTexture2DArray<tsr_ushort> R8Output;

//------------------------------------------------------- GLOBALS
static uint2 GGroupId = 0;

//------------------------------------------------------- DEBUG

static tsr_tensor_half4 Debug[DEBUG_ARRAY_SIZE];

//------------------------------------------------------- INPUT FETCHES

tsr_short2 ComputeElementPixelPos(const uint ElementIndex)
{
	tsr_short2 ClampedFetchPixelPos = ComputeElementInputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);

	return ClampedFetchPixelPos;
}

tsr_tensor_short2 ComputePixelPos()
{
	tsr_tensor_short2 PixelPos;

	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		PixelPos.SetElement(ElementIndex, ComputeElementPixelPos(ElementIndex));
	}

	return PixelPos;
}

tsr_short2 ComputeElementOutputPixelPos(const uint ElementIndex)
{
	return ComputeElementOutputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);
}

void FetchCommonInput(
	tsr_tensor_short2 PixelPos
	,out tsr_tensor_half CurrentCoverage
	,out tsr_tensor_bool bHasPixelAnimation
#if CONFIG_VARYING_RANGE_DETECTION
	,out tsr_tensor_ushort MaterialMask
#endif
	)
{
	tsr_tensor_bool bHasReprojectionOffset;
	tsr_tensor_short2 ReprojectionOffset;

	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		CurrentCoverage.SetElement(ElementIndex, CurrentCoverageTexture[ClampedFetchPixelPos]);

		tsr_ushort EncodedDilateMask = R8Output[int3(ClampedFetchPixelPos,0)];
		bHasReprojectionOffset.SetElement(ElementIndex, (EncodedDilateMask & tsr_ushort(0x80)) != tsr_ushort(0));
		ReprojectionOffset.SetElement(ElementIndex, DecodeReprojectionOffset(R8Output[int3(ClampedFetchPixelPos, ThinGeometryTextureIndex)]));
		tsr_ushort BitMask = tsr_ushort(DecimateMaskTexture[ClampedFetchPixelPos].r * tsr_half(255.0));
		bHasPixelAnimation.SetElement(ElementIndex,(BitMask & tsr_ushort(0x4)) != tsr_ushort(0));
	}

	tsr_tensor_half ReprojectedCurrentCoverage;
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex) + ReprojectionOffset.GetElement(ElementIndex);
		ReprojectedCurrentCoverage.SetElement(ElementIndex, CurrentCoverageTexture[ClampedFetchPixelPos]);
	}
	CurrentCoverage = select(bHasReprojectionOffset, max(CurrentCoverage, ReprojectedCurrentCoverage), tsr_tensor_half::Const(0));

#if CONFIG_VARYING_RANGE_DETECTION
	MaterialMask = tsr_tensor_ushort::CastFrom<tsr_half>(CurrentCoverage * tsr_half(2.01f));
	CurrentCoverage = ceil(CurrentCoverage);
#else
	CurrentCoverage = ToBinaryCoverage(CurrentCoverage);
#endif
}// FetchCommonInput

void FetchRegularPassInput(
	tsr_tensor_short2 PixelPos
	,out tsr_tensor_half SceneDepth
	,out tsr_tensor_half DeviceZThinGeometryDetectionError
	,out tsr_tensor_half ReprojectedHistoryCoverage
#if CONFIG_INTENSITY_LINE_DETECTION
	,out tsr_tensor_half PrevHighIntensity
	,out tsr_tensor_half FlickeringLuma
#endif
#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
	,out tsr_tensor_half HadTranslucency
#endif
)
{
	tsr_tensor_float DepthCache;
#if CONFIG_INTENSITY_LINE_DETECTION || CONFIG_TRANSLUCENCY_COVERAGE_BIT
	tsr_tensor_half  RawCoverageCache;
#endif
	
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
#if CONFIG_INTENSITY_LINE_DETECTION
		FlickeringLuma.SetElement(ElementIndex, FlickeringLumaTexture[ClampedFetchPixelPos]);
		RawCoverageCache.SetElement(ElementIndex, ReprojectedHistoryCoverageTexture[ClampedFetchPixelPos]);
#elif CONFIG_TRANSLUCENCY_COVERAGE_BIT
		RawCoverageCache.SetElement(ElementIndex, ReprojectedHistoryCoverageTexture[ClampedFetchPixelPos]);
#else
		ReprojectedHistoryCoverage.SetElement(ElementIndex, ReprojectedHistoryCoverageTexture[ClampedFetchPixelPos]);
#endif
		DepthCache.SetElement(ElementIndex,SceneDepthTexture[ClampedFetchPixelPos]);
	}

	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		float Depth = DepthCache.GetElement(ElementIndex);
		float DeviceZError = ComputePixelDeviceZError(Depth);
#if		CONFIG_VARYING_RANGE_DETECTION
		DeviceZError *= max(ErrorMultiplier * 0.05f, 1.0f);
		
#else
		DeviceZError *= ErrorMultiplier;
#endif
		SceneDepth.SetElement(ElementIndex, tsr_half(Depth));
		DeviceZThinGeometryDetectionError.SetElement(ElementIndex, tsr_half(DeviceZError));
	}
	
#if CONFIG_INTENSITY_LINE_DETECTION || CONFIG_TRANSLUCENCY_COVERAGE_BIT
	DecodeCoverage(RawCoverageCache, ReprojectedHistoryCoverage	
	#if CONFIG_INTENSITY_LINE_DETECTION
	, PrevHighIntensity
	#endif
	#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
	, HadTranslucency
	#endif
	);
#endif
} // FetchRegularPassInput

//------------------------------------------------------- FUNCTIONS


// Horizontal thin edge detection.
// 0 0 0
// x x x
// 0 0 0

CALL_SITE_DEBUGLOC
template<
typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> HorizontalThinEdge3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> EdgeThreshold)
{

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Top;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Down;
	
	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> One = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(1.0);
	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Zero = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(0);
	

	AccessNeighborTexels1x3(Center, Top, Down);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> TopThinEdge = select(Center - Top > EdgeThreshold, One, Zero);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> DownThinEdge = select(Center - Down > EdgeThreshold, One,Zero);
	
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> PotentialThinGeometryEdge = TopThinEdge + DownThinEdge;
	

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Left;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Right;
	AccessNeighborTexels3x1(PotentialThinGeometryEdge, Left, Right);
#if EDGE_TYPE == EDGE_HORIZONTAL_VERTICAL
	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Five = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(5.0);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>  ThinGeometryEdge = saturate(Left + Right + PotentialThinGeometryEdge - Five);
#elif EDGE_TYPE == EDGE_HORIZONTAL_VERTICAL_DIAGNAL
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> LeftRightSum =  Left + Right;
	LeftRightSum = LeftRightSum * saturate(PotentialThinGeometryEdge - One); // Center pixel should be edge

	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Two = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(2.0);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>  ThinGeometryEdge = saturate((LeftRightSum - Two) * rcp(Two)); // At most one side being edge
#elif
#error EDGE_TYPE not implemented
#endif

	return ThinGeometryEdge;
}

// Vertical thin edge detection.
// 0 x 0
// 0 x 0
// 0 x 0

CALL_SITE_DEBUGLOC
template<
	typename ScalarType, uint VectorSize, uint LaneStrideX, uint LaneStrideY>
TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> VerticalThinEdge3x3(
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Center,
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> EdgeThreshold)
{

	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> One = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(1.0);
	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Zero = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(0);

	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Left;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Right;
	AccessNeighborTexels3x1(Center, Left, Right);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> LeftThinEdge  = select(Center -  Left > EdgeThreshold, One, Zero);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> RightThinEdge = select(Center - Right > EdgeThreshold, One, Zero);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> PotentialThinGeometryEdge = LeftThinEdge + RightThinEdge;


	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Top;
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Bottom;
	AccessNeighborTexels1x3(PotentialThinGeometryEdge, Top, Bottom);
#if EDGE_TYPE == EDGE_HORIZONTAL_VERTICAL
	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Five = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(5.0);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>  ThinGeometryEdge = saturate(Top + Bottom + PotentialThinGeometryEdge - Five);
#elif EDGE_TYPE == EDGE_HORIZONTAL_VERTICAL_DIAGNAL
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> TopBottomSum =  Top + Bottom;
	TopBottomSum = TopBottomSum * saturate(PotentialThinGeometryEdge - One); // Center pixel should be edge

	const TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY> Two = TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>::Const(2.0);
	TLaneVector2D<ScalarType, VectorSize, LaneStrideX, LaneStrideY>  ThinGeometryEdge = saturate((TopBottomSum - Two) * rcp(Two)); // At most one side not being edge
#elif
#error EDGE_TYPE not implemented
#endif

	return ThinGeometryEdge;
}

tsr_tensor_ushort EncodeThinGeometryWeight(tsr_tensor_half ThinGeometryEdge
	, tsr_tensor_bool IsClusterHoleRegion,
	tsr_tensor_half Weight
)
{
	// 7bit   | 1bit
	// weight | 0 : Edge line
	//		  | 1 : Cluster hole region
	Weight = select(IsClusterHoleRegion, Weight, tsr_tensor_half::Const(0));
	Weight = select(ThinGeometryEdge > tsr_tensor_half::Const(0), ThinGeometryEdge, Weight);
	IsClusterHoleRegion = select(ThinGeometryEdge > tsr_tensor_half::Const(0), tsr_tensor_bool::Const(false), IsClusterHoleRegion);
	tsr_tensor_ushort Encoded = bit_shift_left(tsr_tensor_ushort::CastFrom<tsr_half>(Weight * tsr_half(127.0)),tsr_tensor_ushort::Const(tsr_ushort(0x1)));
	Encoded = bit_or(Encoded,select(IsClusterHoleRegion,tsr_tensor_ushort::Const(tsr_ushort(0x1)),tsr_tensor_ushort::Const(0)));
	return Encoded;
}

//------------------------------------------------------- ENTRY POINT

#if COMPILER_SUPPORTS_WAVE_SIZE && DIM_WAVE_SIZE > 0
WAVESIZE(DIM_WAVE_SIZE)
#endif
[numthreads(LANE_COUNT * WAVE_COUNT, 1, 1)]
void DetectThinGeometryCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	GGroupId = GroupId;
	GGroupThreadIndex = GroupThreadIndex;

	#if DEBUG_OUTPUT
	{
		UNROLL_N(DEBUG_ARRAY_SIZE)
		for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
		{
			Debug[DebugId].SetAllElements(0.0);
		}
	}
	#endif

	tsr_tensor_short2 PixelPos = ComputePixelPos();
	tsr_tensor_half DeviceZ;
	tsr_tensor_half DeviceZThinGeometryDetectionError;
	tsr_tensor_half CurrentCoverage;
	tsr_tensor_half ReprojectedHistoryCoverage;
	tsr_tensor_bool bHasPixelAnimation;
	
	#if CONFIG_VARYING_RANGE_DETECTION
	tsr_tensor_ushort MaterialMask;
	#endif
	
	// Fetch data for tiling optimization.
	FetchCommonInput(PixelPos, 
		CurrentCoverage
		,bHasPixelAnimation
	#if CONFIG_VARYING_RANGE_DETECTION
		,MaterialMask
	#endif
	);
		
	bool bRegularPass = AnyThread(AnyComponent(and(CurrentCoverage > tsr_tensor_half::Const(0.0), not(bHasPixelAnimation))));
	tsr_tensor_half ThinGeometryEdge = tsr_tensor_half::Const(0.0);
	tsr_tensor_half HistoryRelaxationWeight = tsr_tensor_half::Const(0);
	tsr_tensor_bool IsClusterHoleRegion = tsr_tensor_bool::Const(false);
	tsr_tensor_half NewSampleWeight = tsr_tensor_half::Const(0.1);
	#if CONFIG_INTENSITY_LINE_DETECTION
	tsr_tensor_half FlickeringLuma = tsr_tensor_half::Const(0);
	tsr_tensor_half PrevHighIntensity = tsr_tensor_half::Const(0);
	tsr_tensor_half IntensityLineEdge = tsr_tensor_half::Const(0);
	tsr_tensor_half CurrentLineEdge = tsr_tensor_half::Const(0);
	#endif
	#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
		tsr_tensor_half HadTranslucency = tsr_tensor_half::Const(0);
	#endif
	// Thin geometry coverage estimation
	if (bRegularPass)
	{
		FetchRegularPassInput(PixelPos,
			DeviceZ, 
			DeviceZThinGeometryDetectionError,
			ReprojectedHistoryCoverage
#if CONFIG_INTENSITY_LINE_DETECTION
			,PrevHighIntensity
			,FlickeringLuma
#endif
#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
			,HadTranslucency
#endif
		);
		
		{
			#if CONFIG_INTENSITY_LINE_DETECTION
			TLineEdgeType LineDetectionValue = Concatenate(DeviceZ, FlickeringLuma);
			TLineEdgeType LineDetectionThreshold = Concatenate(DeviceZThinGeometryDetectionError, tsr_tensor_half::Const(tsr_half(MinKeepLineContrast)));
			#else
			TLineEdgeType LineDetectionValue = DeviceZ;
			TLineEdgeType LineDetectionThreshold = DeviceZThinGeometryDetectionError;
			#endif

			TLineEdgeType LineEdge = HorizontalThinEdge3x3(LineDetectionValue, LineDetectionThreshold);
			LineEdge = LineEdge + VerticalThinEdge3x3(LineDetectionValue, LineDetectionThreshold);
			//Debug[0].SetComponent(2, tsr_tensor_half(ThinGeometryEdge));
			#if CONFIG_INTENSITY_LINE_DETECTION
			ThinGeometryEdge = LineEdge[0];
			CurrentLineEdge = saturate(ceil(LineEdge[1]));
			//Debug[0].SetComponent(1, tsr_tensor_half(CurrentLineEdge));
			#else
			ThinGeometryEdge = LineEdge;
			#endif
		}

		// Student t test of current observation and history to check if they are of the same distribution.
		tsr_tensor_bool bSameDistribution = tsr_tensor_bool::Const(false);
		tsr_tensor_bool NotAbsoluteMatch = tsr_tensor_bool::Const(false);
		tsr_tensor_half T;
		{ // Student T test on Bernouli distribution of 3x3 neighbours. If T > T critical, reject weights due to low similarity.

			tsr_tensor_half3 ExpectObservationAndVariance = Concatenate(Concatenate(ReprojectedHistoryCoverage,CurrentCoverage),ReprojectedHistoryCoverage * (tsr_tensor_half::Const(1.0) - ReprojectedHistoryCoverage));
			ExpectObservationAndVariance = Sum3x3(ExpectObservationAndVariance);

			tsr_tensor_half XAverage = ExpectObservationAndVariance[0];
			XAverage = (XAverage - ExpectObservationAndVariance[1]);
			tsr_tensor_half S = sqrt(ExpectObservationAndVariance[2] + tsr_tensor_half::Const(tsr_half(0.000001)))
				* sqrt(NewSampleWeight * rcp(tsr_tensor_half::Const(2.0) - NewSampleWeight) * tsr_half(72.0));
			// When observation is equivalent to the expectation, t = 0.
			T = abs(select( XAverage > tsr_tensor_half::Const(0), XAverage * rcp(S), tsr_tensor_half::Const(0)));
			bSameDistribution = T < tsr_tensor_half::Const(20.754); //2.306 (p=0.05) * 9
			
			//Debug[0].SetComponent(0,abs(T));
			//Debug[0].SetComponent(1,tsr_tensor_half::CastFrom<bool>(abs(T) > tsr_tensor_half::Const(20.754)));
		}

		tsr_tensor_bool ValidCurrentThinGeometryRegion = tsr_tensor_bool::Const(false);
		{
			tsr_tensor_half2 TAndCoverage = Concatenate(T, CurrentCoverage);
			TAndCoverage = Max3x3(TAndCoverage);
			TAndCoverage = Max3x3(TAndCoverage);
			NotAbsoluteMatch =TAndCoverage[0]/* Max3x3(Max3x3(T))*/ > tsr_tensor_half::Const(0);
			ValidCurrentThinGeometryRegion = TAndCoverage[1]/*Max3x3(Max3x3(CurrentCoverage))*/ > tsr_tensor_half::Const(0);
		}

		// Sample weight for coverage update.
		{
			// Reject history coverage if history is not valid or absolute match or has pixel animation.
			// TODO: Absolute coverage match might has similar color, then no need to reset history coverage.
			// e.g., 5x5 flat surface disappear and 5x5 flat surface appear. This is uncommon in thin geometry
			// yet need to address to be less aggressive.
			tsr_tensor_bool AcceptHistory = and(ValidCurrentThinGeometryRegion, and(bSameDistribution, and(NotAbsoluteMatch, not(bHasPixelAnimation))));
			NewSampleWeight = select(AcceptHistory, NewSampleWeight, tsr_tensor_half::Const(1.0));

			//1. Update reprojected history and use it for weight calculation.
			ReprojectedHistoryCoverage = lerp(ReprojectedHistoryCoverage, CurrentCoverage, NewSampleWeight);
			//Debug[0].SetComponent(1, NewSampleWeight);

			// Valid pixels for cluster hole stability improvement.
			IsClusterHoleRegion = ReprojectedHistoryCoverage > tsr_tensor_half::Const(half(0.99f/127.0f));

			#if CONFIG_INTENSITY_LINE_DETECTION
			{
				// 2. stochastic OR accumulation of intensity lines
				// Keep it live through bright-dark alternation
				// stochastically decay clears it after consecutive dark frames
				tsr_tensor_half Noise;
				UNROLL_N(SIMD_SIZE)
				for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
				{
					//Cannot use View.StateFrameIndexMod8 as it cannot trigger decay with a random initialization
					#if 0
						uint2 Random = Rand3DPCG16(int3(float2(PixelPos.GetElement(ElementIndex)), View.StateFrameIndex)).xy;
						tsr_half TmpNoise = tsr_half(Hammersley16(0, 1, Random).x);
					#else
						tsr_half TmpNoise = tsr_half(InterleavedGradientNoise(float2(PixelPos.GetElement(ElementIndex)), View.StateFrameIndex));
					#endif
				
					Noise.SetElement(ElementIndex, TmpNoise);
				}

				// low chance to decay if current is not high contrast line and inside region (and higher if it is outside the region).  
				tsr_tensor_half bDark = tsr_tensor_half::Const(tsr_half(1.0)) - CurrentLineEdge;
				tsr_tensor_half DecayProbability = select(IsClusterHoleRegion, 
					tsr_tensor_half::Const(tsr_half(HighContrastLineFadeRateInsideRegion)), 
					tsr_tensor_half::Const(tsr_half(HighContrastLineFadeRate)));
				tsr_tensor_half bDecay = bDark * saturate((DecayProbability - Noise) * tsr_tensor_half::Const(tsr_half(POSITIVE_INFINITY)));

				//Debug[0].SetComponent(0, bDecay);
				//Debug[0].SetComponent(3, PrevHighIntensity);

				IntensityLineEdge = max(CurrentLineEdge, PrevHighIntensity) * (tsr_tensor_half::Const(1.0) - bDecay);
			}
			#endif
		}

		// Update new history relaxation weight (thin geometry cluster and Intensity weights).
		{
			// Calculate the weights.
			{
				HistoryRelaxationWeight = abs(ReprojectedHistoryCoverage - tsr_tensor_half::Const(0.5))+tsr_tensor_half::Const(0.02);
				HistoryRelaxationWeight = HistoryRelaxationWeight * exp(HistoryRelaxationWeight * tsr_half(-20.0)) *  tsr_half(10.0/0.184); // delta = 1 maximum
			}
			#if CONFIG_INTENSITY_LINE_DETECTION
				// Min5x5(1 - x) == 1 - Max5x5(x), so this computes:
				//   Min5x5HistoryCoverage = min5x5(ReprojectedHistoryCoverage)
				//   IntensityWeight       = max5x5(IntensityLineEdge)  (after 1 - Min5x5(1 - x))
				tsr_tensor_half Min5x5HistoryCoverage;
				tsr_tensor_half IntensityWeight;
				Deconcatenate(Min3x3(Min3x3(Concatenate(ReprojectedHistoryCoverage, tsr_tensor_half::Const(1.0) - IntensityLineEdge))), Min5x5HistoryCoverage, IntensityWeight);
				IntensityWeight = tsr_tensor_half::Const(1.0) - IntensityWeight;
			#else
				tsr_tensor_half Min5x5HistoryCoverage = Min3x3(Min3x3(ReprojectedHistoryCoverage));
			#endif
			
			tsr_tensor_bool bThinGeometryHistoryRegion = Min5x5HistoryCoverage > tsr_tensor_half::Const(0);
			
			// Reject samples that has min being background
			// Set relaxation weights to 0 if the history is rejected to avoid overblur.
			tsr_tensor_bool bValidHistoryRelaxation = and(bThinGeometryHistoryRegion,and(ValidCurrentThinGeometryRegion, bSameDistribution));
			HistoryRelaxationWeight = select(bValidHistoryRelaxation, HistoryRelaxationWeight, tsr_tensor_half::Const(0));
			
			#if CONFIG_VARYING_RANGE_DETECTION
				tsr_tensor_half NonFoliageThinGeometryEdge = ThinGeometryEdge;
				tsr_tensor_half NonFoliageRelaxationWeight = HistoryRelaxationWeight;
				bool bAnyFoliages = AnyThread(AnyComponent(and(MaterialMask == tsr_tensor_ushort::Const(2), not(bHasPixelAnimation))));
			if (bAnyFoliages)
			#endif
			//1. Diffuse a little bit to suppress the history clamping around the valid pixel and apply max relaxation weight.
			//2. Dilate the edge with (0.8,1,0.8), such that, dilated edges has 0.8 for direct neighbor, 0.64 for diagnal, 1 for consecutive neighbors.
			{
				Deconcatenate(WeightedAvg3x3(Concatenate(ThinGeometryEdge, HistoryRelaxationWeight), float3(0.8, 1.0, 0.8)), ThinGeometryEdge, HistoryRelaxationWeight);
				HistoryRelaxationWeight = saturate(WeightedAvg3x3(saturate(HistoryRelaxationWeight * tsr_half(MaxRelaxationWeight)), float3(0.5, 1.0, 0.5)));
				
				#if CONFIG_VARYING_RANGE_DETECTION

				// Regular non-foliage materials does not require extensive relaxation when temporal flickering logic is active.
				{
					tsr_tensor_bool bNonFoliages = MaterialMask == tsr_tensor_ushort::Const(1);

					ThinGeometryEdge = select(bNonFoliages, NonFoliageThinGeometryEdge, ThinGeometryEdge);
					HistoryRelaxationWeight = select(bNonFoliages, NonFoliageRelaxationWeight, HistoryRelaxationWeight);
				}
				#endif
			}
			
			#if CONFIG_INTENSITY_LINE_DETECTION
			// Update intensity weights
			// Apply a uniform weight when intensity line antialiasing is required and HistoryRelaxationWeight is not set.
			{
				IntensityWeight = IntensityWeight * (tsr_tensor_half::Const(tsr_half(1)) - CurrentLineEdge);
				// cluster weights is more important than intensity weights.
				HistoryRelaxationWeight = select(HistoryRelaxationWeight > tsr_tensor_half::Const(0), HistoryRelaxationWeight, IntensityWeight * tsr_tensor_half::Const(tsr_half(HighContrastLineWeight)));
				//Debug[0].SetComponent(2,IntensityLineEdge);
			}
			#endif

			#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
			{
				// Do not apply relaxation if previous frame translucency has high contribution.
				tsr_tensor_half TranslucencyRejection = (tsr_tensor_half::Const(tsr_half(1)) - HadTranslucency);
				HistoryRelaxationWeight = HistoryRelaxationWeight * TranslucencyRejection;
				ThinGeometryEdge = ThinGeometryEdge * TranslucencyRejection;
				#if CONFIG_INTENSITY_LINE_DETECTION
				IntensityLineEdge = IntensityLineEdge * TranslucencyRejection;
				#endif
			}
			#endif

			// Not similar patches or has pixel animation should reset relaxation weight to 0
			// to use more strict history clamping.
			HistoryRelaxationWeight = select(and(bSameDistribution, not(bHasPixelAnimation)), HistoryRelaxationWeight, tsr_tensor_half::Const(0));
			ThinGeometryEdge = select(bHasPixelAnimation, tsr_tensor_half::Const(0), ThinGeometryEdge);
			//Debug[0].SetComponent(2,HistoryRelaxationWeight);
		}

	}
	else
	{	// Fast pass does not have any thin geometry indicators. Update coverage to current.
		ReprojectedHistoryCoverage = CurrentCoverage;
	}

	tsr_tensor_ushort EncodedThinGeometryEdge = EncodeThinGeometryWeight(
		ThinGeometryEdge
		,IsClusterHoleRegion
		,HistoryRelaxationWeight
	);

	// Output the thin geometry mask
	UNROLL_N(SIMD_SIZE)
	for(uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		const tsr_short2 LaneSimdPixelOffset = GetElementPixelOffsetInTile(LANE_STRIDE_X, LANE_STRIDE_Y, GGroupThreadIndex, ElementIndex);
		const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
		
		R8Output[uint3(OutputPixelPos,ThinGeometryTextureIndex)] = EncodedThinGeometryEdge.GetElement(ElementIndex);//uint(ThinGeometryFlag * tsr_half(255.0));
		
		tsr_half Dither = tsr_half(InterleavedGradientNoise(float2(OutputPixelPos), View.StateFrameIndex));

		#if CONFIG_INTENSITY_LINE_DETECTION
			tsr_half CoverageClamped = saturate(ReprojectedHistoryCoverage.GetElement(ElementIndex));
			bool bHighContrastLine = IntensityLineEdge.GetElement(ElementIndex) > tsr_half(0);
			HistoryCoverageOutput[uint3(OutputPixelPos, 0)] = EncodeCoverage(CoverageClamped, bHighContrastLine, Dither);
		#else
			HistoryCoverageOutput[uint3(OutputPixelPos, 0)] = EncodeCoverage(ReprojectedHistoryCoverage.GetElement(ElementIndex), false, Dither);
		#endif
	}
	
	#if DEBUG_OUTPUT
	{
		// Mark the top left corner of the tile for debugging purposes.
		#if DEBUG_DISABLE_PADDING == 2
		if (GGroupThreadIndex == 0)
		{
			UNROLL_N(DEBUG_ARRAY_SIZE)
			for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
			{
				Debug[DebugId].SetElement(/* ElementIndex = */ 0, 1.0);
			}
		}
		#endif // DEBUG_DISABLE_PADDING == 2
	
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(
				/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
				TileOverscan,
				InputInfo_ViewportMin,
				InputInfo_ViewportMax,
				GGroupId,
				GGroupThreadIndex,
				ElementIndex,
				/* OutputDataOffset = */ int2(0, 0),
				/* OutputResDivisor = */ 1,
				/* bDebugDisablePadding = */ DEBUG_DISABLE_PADDING != 0);

			UNROLL_N(DEBUG_ARRAY_SIZE)
			for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
			{
				DebugOutput[tsr_short3(OutputPixelPos, DebugId)] = float4(Debug[DebugId].GetElement(ElementIndex));
			}
		}
	}
	#endif // DEBUG_OUTPUT
	
}
