// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRDepthVelocityAnalysis.ush"
#include "TSRSpatialAntiAliasing.ush"
#include "TSRReprojectionField.ush"
#include "TSRClosestOccluder.ush"

#if DIM_MOTION_BLUR_DIRECTIONS > 0
	#define CONFIG_MAX_RANGE_SIZE (DIM_MOTION_BLUR_DIRECTIONS)
	#include "../MotionBlur/MotionBlurVelocityFlatten.ush"
#endif


//------------------------------------------------------- CONFIG

#if DIM_MOTION_BLUR_DIRECTIONS > 0
	#define TILE_SIZE (VELOCITY_FLATTEN_TILE_SIZE)
#else
	#define TILE_SIZE 8
#endif

#define CONFIG_THIN_GEOMETRY_EDGE_REPROJECTION (DIM_THIN_GEOMETRY_EDGE_REPROJECTION)

//------------------------------------------------------- PARAMETERS

float4x4 RotationalClipToPrevClip;
float InvFlickeringMaxParralaxVelocity;
float ReprojectionFieldAntiAliasVelocityThreshold;
uint bReprojectionField;
uint bOutputIsMovingTexture;
uint ReprojectionVectorOutputIndex;
#if CONFIG_THIN_GEOMETRY_EDGE_REPROJECTION
uint ThinGeometryTextureIndex;
#endif

Texture2D<float> SceneDepthTexture;
Texture2D<float4> SceneVelocityTexture;

RWTexture2D<float2> ClosestDepthOutput;
globallycoherent RWTexture2DArray<uint> PrevAtomicOutput;
RWTexture2DArray<uint> ReprojectionFieldOutput;
RWTexture2DArray<uint> R8Output;

RWTexture2D<float3>	VelocityFlattenOutput;
RWTexture2DArray<float4> VelocityTileArrayOutput;


//------------------------------------------------------- DEBUG

#define DEBUG_ARRAY_SIZE 8

static float4 Debug[DEBUG_ARRAY_SIZE];


//------------------------------------------------------- FUNCTIONS

/** Spatial anti-alias the depth buffer and return the boundary. */
tsr_half2 ComputeReprojectionBoundary(
	tsr_short2 InputPixelPos,
	tsr_short2 ReprojectionOffset,
	float2 ScreenVelocity,
	float2 DilatedScreenVelocity,
	float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES])
{
	const uint BrowsingIterations = 3;

	float InputC = AccessNeighborhoodCenter(DevizeZNeighborhood);
		
	float InputN = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetN);
	float InputS = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetS);
	float InputE = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetE);
	float InputW = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetW);
		
	float InputNE = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetNE);
	float InputNW = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetNW);
	float InputSE = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetSE);
	float InputSW = AccessNeighborhoodStaticOffset(DevizeZNeighborhood, kOffsetSW);
	
	float2 ScreenPos = ApplyScreenTransform(float2(int2(InputPixelPos)), InputPixelPosToScreenPos);

	float2 PrevScreenPos = ScreenPos - DilatedScreenVelocity.xy;
	bool bOffscreen = any(PrevScreenPos != clamp(PrevScreenPos, -1.0, 1.0));

	float2 PixelVelocityDifferential = InputInfo_ViewportSize * (ScreenVelocity.xy - DilatedScreenVelocity.xy);

	float PixelVelocity = length2(PixelVelocityDifferential);
	bool bDilationHasEnoughVelocityDiff = PixelVelocity > ReprojectionFieldAntiAliasVelocityThreshold;

	bool bAntiAliasReprojectionField = bReprojectionField && bDilationHasEnoughVelocityDiff && !bOffscreen;

	tsr_half2 ReprojectionBoundary;
	BRANCH
	if (bAntiAliasReprojectionField)
	{
		float NoiseFiltering;
		tsr_short2 BrowseDirection;
		tsr_short2 EdgeSide;
		float EdgeInput;
		FindBrowsingDirection(
			InputC,
			InputN,  InputS,  InputE,  InputW,
			InputNE, InputNW, InputSE, InputSW,
			/* out */ NoiseFiltering,
			/* out */ BrowseDirection,
			/* out */ EdgeSide,
			/* out */ EdgeInput);
				
		#if CONFIG_BUTTERFLY_KERNEL
			EdgeSide *= GetLaneOffsetSign();
		#endif
		
		tsr_ushort EdgeLengthP, EdgeLengthN;
		bool bEdgeStopedByIncrementP, bEdgeStopedByIncrementN;
		bool bEdgeStopedByDecrementP, bEdgeStopedByDecrementN;
		BrowseNeighborhoodBilinearOptimized(
			BrowsingIterations,
			SceneDepthTexture,
			InputC,
			EdgeInput,
			InputPixelPos,
			BrowseDirection,
			EdgeSide,
			/* out */ EdgeLengthP,
			/* out */ EdgeLengthN,
			/* out */ bEdgeStopedByIncrementP,
			/* out */ bEdgeStopedByIncrementN,
			/* out */ bEdgeStopedByDecrementP,
			/* out */ bEdgeStopedByDecrementN);
			
		ReprojectionBoundary = ComputeReprojectionBoundary(
			BrowsingIterations,
			EdgeSide, EdgeLengthP, EdgeLengthN,
			bEdgeStopedByIncrementP, bEdgeStopedByIncrementN,
			bEdgeStopedByDecrementP, bEdgeStopedByDecrementN);
	}
	else
	{
		// Fully dilate the boundary in the history update to still apply the jacobian on geometric edges.
		ReprojectionBoundary = kFullDilateBoundary;
	}

	return ReprojectionBoundary;
} // ComputeReprojectionBoundary()


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void DilateVelocityCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
	{
		Debug[DebugId] = 0.0;
	}

	uint2 DispatchThreadId = (
		ZOrder2D(GroupThreadIndex, uint(log2(float(TILE_SIZE)))) +
		GroupId * uint2(TILE_SIZE, TILE_SIZE));

	tsr_short2 InputPixelPos = tsr_short2(InputInfo_ViewportMin + DispatchThreadId);
	
	// Find the closest depth and its offset.
	float DeviceZ;
	float ClosestDeviceZ;
	float3 DilatedScreenVelocity;
	uint EncodedReprojectionBoundary;
	uint EncodedReprojectionJacobian;
	uint EncodedDeviceZError;
	bool bHasPixelAnimation;
	bool bDisableAnimationAA;
	bool bHasReprojectionOffset;
	tsr_half ReprojectionEdge;
	BRANCH
	if (bReprojectionField)
	{
		// Load the 3x3 Neighborhood.
		float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		float4 VelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		ISOLATE
		{
			FetchDepthVelocity3x3(SceneDepthTexture, SceneVelocityTexture, InputPixelPos, /* out */ DevizeZNeighborhood, /* out */ VelocityNeighborhood);
		}

		// Convert velocities to pixel velocities to take down register pressure.
		float2 PixelVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		float  DepthVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		bool   bIsDrawingVelocityNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		ISOLATE
		{
			ComputePixelVelocityNeighborhood(
				InputPixelPos,
				DevizeZNeighborhood,
				VelocityNeighborhood,
				/* out */ PixelVelocityNeighborhood,
				/* out */ DepthVelocityNeighborhood,
				/* out */ bIsDrawingVelocityNeighborhood);
		}
		#if VELOCITY_ENCODE_HAS_PIXEL_ANIMATION
			bHasPixelAnimation = DecodeHasPixelAnimationFromVelocityTexture(AccessNeighborhoodCenter(VelocityNeighborhood));
		#endif
		#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
			bDisableAnimationAA = DecodeDisableAnimationAAFromVelocityTexture(AccessNeighborhoodCenter(VelocityNeighborhood));
		#endif
	
		tsr_half2x2 ReprojectionJacobian;
		float ParallaxDepthError;
		ISOLATE
		{
			ComputeReprojectionJacobian(
				DevizeZNeighborhood,
				PixelVelocityNeighborhood,
				/* out */ ReprojectionJacobian,
				/* out */ ParallaxDepthError);
			EncodedReprojectionJacobian = EncodeReprojectionJacobian(ReprojectionJacobian);
		}
		
		tsr_short2 ReprojectionOffset;
		ISOLATE
		{
			FindClosestDepthOffset(
				DevizeZNeighborhood,
				/* out */ ClosestDeviceZ,
				/* out */ ReprojectionOffset);
			bHasReprojectionOffset = any(ReprojectionOffset != tsr_short(0));
;		}

		float2 ScreenVelocity;
		ISOLATE
		{
			// Compute final screen vecocity from pixel velocities to avoid duplicated register.
			DeviceZ = AccessNeighborhoodCenter(DevizeZNeighborhood);
			ScreenVelocity = AccessNeighborhoodCenter(PixelVelocityNeighborhood) * InputPixelVelocityToScreenVelocity;
	
			// Compute the dilated screen velocity.
			bool bDilatedHasPixelAnimation;
			bool bDilatedDisableAnimationAA;
			FetchAndComputeScreenVelocity(
				SceneVelocityTexture,
				InputPixelPos,
				ClosestDeviceZ,
				ReprojectionOffset,
				/* out */ DilatedScreenVelocity,
				/* out */ bDilatedHasPixelAnimation,
				/* out */ bDilatedDisableAnimationAA);
		}
		
		ReprojectionEdge = ComputeReprojectionEdge(ScreenVelocity, DilatedScreenVelocity.xy);
		
		// Compute the reprojection boundary using a depth spatial anti-aliaser
		ISOLATE
		{
			tsr_half2 ReprojectionBoundary = ComputeReprojectionBoundary(
				InputPixelPos,
				ReprojectionOffset,
				ScreenVelocity,
				DilatedScreenVelocity.xy,
				DevizeZNeighborhood);

			EncodedReprojectionBoundary = EncodeReprojectionBoundary(ReprojectionOffset, ReprojectionBoundary);
		}
		
		// Encode the device z error.
		{
			float PrevClosestDeviceZ = ClosestDeviceZ - DilatedScreenVelocity.z;
			EncodedDeviceZError = EncodeDeviceZError(PrevClosestDeviceZ, ParallaxDepthError);
		}
	}
	else // if (!bReprojectionField)
	{
		// Load the 3x3 Neighborhood.
		float DevizeZNeighborhood[CONFIG_BUTTERFLY_SAMPLES];
		ISOLATE
		{
			FetchDepth3x3(SceneDepthTexture, InputPixelPos, /* out */ DevizeZNeighborhood);
		}
		
		tsr_short2 ReprojectionOffset;
		{
			FindClosestDepthOffset(
				DevizeZNeighborhood,
				/* out */ ClosestDeviceZ,
				/* out */ ReprojectionOffset);
			bHasReprojectionOffset = any(ReprojectionOffset != tsr_short(0));
		}
	
		// Compute the parallax depth error
		float ParallaxDepthError = ComputeDepthError(DevizeZNeighborhood);

		// Compute final screen vecocity from pixel velocities to avoid duplicated register.
		DeviceZ = AccessNeighborhoodCenter(DevizeZNeighborhood);
	
		// Compute the screen vecocity.
		float3 ScreenVelocity;
		FetchAndComputeScreenVelocity(
			SceneVelocityTexture,
			InputPixelPos,
			DeviceZ,
			/* ReprojectionOffset = */ tsr_short2(0, 0),
			/* out */ ScreenVelocity,
			/* out */ bHasPixelAnimation,
			/* out */ bDisableAnimationAA);

		// Compute the dilated screen vecocity.
		bool bDilatedHasPixelAnimation;
		bool bDilatedDisableAnimationAA;
		FetchAndComputeScreenVelocity(
			SceneVelocityTexture,
			InputPixelPos,
			ClosestDeviceZ,
			ReprojectionOffset,
			/* out */ DilatedScreenVelocity,
			/* out */ bDilatedHasPixelAnimation,
			/* out */ bDilatedDisableAnimationAA);
		
		ReprojectionEdge = ComputeReprojectionEdge(ScreenVelocity.xy, DilatedScreenVelocity.xy);
		
		// Encode the device z error.
		{
			float PrevClosestDeviceZ = ClosestDeviceZ - DilatedScreenVelocity.z;
			EncodedDeviceZError = EncodeDeviceZError(PrevClosestDeviceZ, ParallaxDepthError);
		}

#if CONFIG_THIN_GEOMETRY_EDGE_REPROJECTION
		EncodedReprojectionBoundary = EncodeReprojectionOffset(ReprojectionOffset);
#else
		EncodedReprojectionBoundary = 0u;
#endif
		EncodedReprojectionJacobian = 0u;
	}

	tsr_half IsMovingMask = tsr_half(0.0);
	BRANCH
	if (bOutputIsMovingTexture)
	{
		float2 ScreenPos = ApplyScreenTransform(float2(int2(InputPixelPos)), InputPixelPosToScreenPos);
		float Depth = ConvertFromDeviceZ(ClosestDeviceZ);

		float PrevDepth = ConvertFromDeviceZ(ClosestDeviceZ - DilatedScreenVelocity.z);
		float2 PrevScreenPos = ScreenPos - DilatedScreenVelocity.xy;
		
		float4 ThisClip = float4(ScreenPos, DeviceZ, 1);
		float4 PrevClip = mul(ThisClip, View.ClipToPrevClip); 
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
			
		float4 RotationalPrevClip = mul(ThisClip, RotationalClipToPrevClip);
		float2 RotationalPrevScreen = RotationalPrevClip.xy / RotationalPrevClip.w;
			
		tsr_half PixelParallaxVelocity = tsr_half(0.5) * length(tsr_half2(RotationalPrevScreen - PrevScreen) * tsr_half2(InputInfo_ViewportSize));
			
		float4 ClipPosition = float4(GetScreenPositionForProjectionType(ScreenPos,Depth), Depth, 1);
		float4 PrevClipPosition = float4(GetScreenPositionForProjectionType(PrevScreenPos, PrevDepth), PrevDepth, 1);
		
		float3 PreViewTranslationOffset = DFFastLocalSubtractDemote(PrimaryView.PreViewTranslation, PrimaryView.PrevPreViewTranslation);
		float3 TranslatedWorldPosition = mul(ClipPosition, View.ScreenToTranslatedWorld).xyz;
		float3 PrevTranslatedWorldPosition = mul(PrevClipPosition, View.PrevScreenToTranslatedWorld).xyz + PreViewTranslationOffset;
		
		float StaticWorldRadius = GetDepthPixelRadiusForProjectionType(Depth) * 2.0f;

		IsMovingMask = max(
			tsr_half(saturate(length(TranslatedWorldPosition - PrevTranslatedWorldPosition) * rcp(StaticWorldRadius) - 1.0)),
			saturate(PixelParallaxVelocity * tsr_half(InvFlickeringMaxParralaxVelocity) - tsr_half(0.5)));

		IsMovingMask = select(bHasPixelAnimation, tsr_half(1.0), IsMovingMask);
	}

	// Do motion blur velocity flatten.
	#if DIM_MOTION_BLUR_DIRECTIONS > 0
		FVelocityRange VelocityPolarRange;
		float2 VelocityPolar;
		ReduceVelocityFlattenTile(GroupThreadIndex, DilatedScreenVelocity.xy, /* out */ VelocityPolarRange, /* out */ VelocityPolar);
	#endif
	
	float PrevClosestDeviceZ = ClosestDeviceZ - DilatedScreenVelocity.z;

	{
		bool bValidOutputPixel = all(InputPixelPos < InputInfo_ViewportMax);
		uint2 OutputPixelPos = bValidOutputPixel ? InputPixelPos : uint(~0).xx;

		ClosestDepthOutput[OutputPixelPos] = float2(select(bHasPixelAnimation, -PrevClosestDeviceZ, PrevClosestDeviceZ),
		#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS 
													select(bDisableAnimationAA, -ClosestDeviceZ, ClosestDeviceZ)
		#else
													ClosestDeviceZ
		#endif
													);

		R8Output[uint3(OutputPixelPos, 0)] = uint(round(ReprojectionEdge * 127.0)) | select(bHasReprojectionOffset, 0x80u, 0x00u);
		R8Output[uint3(OutputPixelPos, 1)] = EncodedDeviceZError;

        #if CONFIG_THIN_GEOMETRY_EDGE_REPROJECTION
		    R8Output[uint3(OutputPixelPos, ThinGeometryTextureIndex)] = EncodedReprojectionBoundary & 0x0fu;
        #endif

		BRANCH
		if (bOutputIsMovingTexture)
		{
			R8Output[uint3(OutputPixelPos, 2)] = uint(round(IsMovingMask * 255.0));
		}

		// Output the reprojection field
		ReprojectionFieldOutput[uint3(OutputPixelPos, ReprojectionVectorOutputIndex)] = EncodeReprojectionVector(DilatedScreenVelocity.xy);
		BRANCH
		if (bReprojectionField)
		{
			ReprojectionFieldOutput[uint3(OutputPixelPos, kReprojectionJacobianOutputIndex)] = EncodedReprojectionJacobian;
			ReprojectionFieldOutput[uint3(OutputPixelPos, kReprojectionBoundaryOutputIndex)] = EncodedReprojectionBoundary;
		}

		// Output motion blur velocity flatten
		#if DIM_MOTION_BLUR_DIRECTIONS > 0
		{
			VelocityFlattenOutput[OutputPixelPos] = EncodeVelocityFlatten(VelocityPolar, ClosestDeviceZ);
			
			uint2 OutputTilePos = GroupThreadIndex == 0 ? GroupId : uint(~0).xx;
			StoreVelocityRange(VelocityTileArrayOutput, OutputTilePos, PolarToCartesian(VelocityPolarRange));
		}
		#endif
		
		// Scatter parallax rejection
		float2 ScreenPos = ApplyScreenTransform(float2(int2(OutputPixelPos)), InputPixelPosToScreenPos);
		ScatterClosestOccluder(PrevAtomicOutput, bValidOutputPixel, ScreenPos, DilatedScreenVelocity.xy, PrevClosestDeviceZ);
		
		#if DEBUG_OUTPUT
		for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
		{
			DebugOutput[tsr_short3(OutputPixelPos, DebugId)] = Debug[DebugId];
		}
		#endif
	}
}
