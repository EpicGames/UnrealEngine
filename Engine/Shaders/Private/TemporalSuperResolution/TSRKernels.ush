// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRCommon.ush"


//------------------------------------------------------- CONSTANTS

// K = Center of the nearest input pixel.
// O = Center of the output pixel.
//
//          |           |
//    0     |     1     |     2
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          | O         |
//    3     |     K     |     5
//          |           |
//          |           |
//  --------+-----------+--------
//          |           |
//          |           |
//    6     |     7     |     8
//          |           |
//
static const int2 kOffsets3x3[9] =
{
	int2(-1, -1),
	int2(0, -1),
	int2(1, -1),
	int2(-1,  0),
	int2(0,  0), // K
	int2(1,  0),
	int2(-1,  1),
	int2(0,  1),
	int2(1,  1),
};

// T = Center of the nearest top left pixel input pixel.
// O = Center of the output pixel.
//
//          | 
//    T     |     .
//          | 
//       O  | 
//  --------+--------
//          | 
//          | 
//    .     |     .
//          | 
static const int2 Offsets2x2[4] =
{
	int2( 0,  0), // T
	int2( 1,  0),
	int2( 0,  1),
	int2( 1,  1),
};

// Indexes of the 3x3 square.
static const uint kSquareIndexes3x3[9] = { 4, 0, 1, 2, 3, 8, 7, 6, 5 };

// Indexes of the offsets to have plus + shape.
static const uint kPlusIndexes3x3[5] = { 4, 1, 3, 7, 5 };

// Cardinal pixel directions;
static const int2 kOffsetC = int2( 0,  0);
static const int2 kOffsetN = int2( 0, -1);
static const int2 kOffsetS = int2( 0, +1);
static const int2 kOffsetE = int2(+1,  0);
static const int2 kOffsetW = int2(-1,  0);
static const int2 kOffsetNE = int2(+1, -1);
static const int2 kOffsetNW = int2(-1, -1);
static const int2 kOffsetSE = int2(+1, +1);
static const int2 kOffsetSW = int2(-1, +1);

static const int2 kPairOffsets[4] = {
	int2( 1,  0),
	int2( 1,  1),
	int2( 0,  1),
	int2(-1,  1),
};


//------------------------------------------------------- CONFIG

#if PLATFORM_SUPPORTS_WAVE_BROADCAST && 1
	#define CONFIG_BUTTERFLY_KERNEL  1
	#define CONFIG_BUTTERFLY_SAMPLES 4
#else
	#define CONFIG_BUTTERFLY_KERNEL  0
	#define CONFIG_BUTTERFLY_SAMPLES 9
#endif


//------------------------------------------------------- PIXEL OFFSET CLAMPING

// Clamp the offset to be shared across multiple samples
CALL_SITE_DEBUGLOC
tsr_short2x2 ClampPixelOffset(
	tsr_short2x2 KernelCenterPixelPos,
	tsr_short2x2 Offset,
	const tsr_short2 OffsetDirection,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	tsr_short2x2 Min = dpv_sub(tsr_short2(MinPixelPos), KernelCenterPixelPos);
	tsr_short2x2 Max = dpv_sub(tsr_short2(MaxPixelPos), KernelCenterPixelPos);

	// Only do clamp based on the compile time known direction of the offset.
	// This turns only 1 v_pk_max_i16 and v_pk_min_i16 for a 3x3 kernel.
	tsr_short2x2 ClampedOffset = 0;

	if (OffsetDirection.x > 0)
	{
		ClampedOffset[0] = min(Offset[0], Max[0]);
	}
	else if (OffsetDirection.x < 0)
	{
		ClampedOffset[0] = max(Offset[0], Min[0]);
	}
		
	if (OffsetDirection.y > 0)
	{
		ClampedOffset[1] = min(Offset[1], Max[1]);
	}
	else if (OffsetDirection.y < 0)
	{
		ClampedOffset[1] = max(Offset[1], Min[1]);
	}

	return ClampedOffset;
}

CALL_SITE_DEBUGLOC
tsr_short2 ClampPixelOffset(
	tsr_short2 KernelCenterPixelPos,
	tsr_short2 Offset,
	const tsr_short2 OffsetDirection,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	tsr_short2 Min = tsr_short2(MinPixelPos) - KernelCenterPixelPos;
	tsr_short2 Max = tsr_short2(MaxPixelPos) - KernelCenterPixelPos;

	// Only do clamp based on the compile time known direction of the offset.
	// This turns only 1 v_pk_max_i16 and v_pk_min_i16 for a 3x3 kernel.
	tsr_short2 ClampedOffset = 0;

	if (OffsetDirection.x > 0)
		ClampedOffset.x = min(Offset.x, Max.x);
	else if (OffsetDirection.x < 0)
		ClampedOffset.x = max(Offset.x, Min.x);
		
	if (OffsetDirection.y > 0)
		ClampedOffset.y = min(Offset.y, Max.y);
	else if (OffsetDirection.y < 0)
		ClampedOffset.y = max(Offset.y, Min.y);

	return ClampedOffset;
}

CALL_SITE_DEBUGLOC
tsr_short2x2 ClampPixelOffset(
	tsr_short2x2 SamplePixelPos,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	SamplePixelPos[0] = fastClamp(SamplePixelPos[0], tsr_short2(MinPixelPos.xx), tsr_short2(MaxPixelPos.xx));
	SamplePixelPos[1] = fastClamp(SamplePixelPos[1], tsr_short2(MinPixelPos.yy), tsr_short2(MaxPixelPos.yy));
	return SamplePixelPos;
}

CALL_SITE_DEBUGLOC
tsr_short2 ClampPixelOffset(
	tsr_short2 SamplePixelPos,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	return fastClamp(SamplePixelPos, tsr_short2(MinPixelPos), tsr_short2(MaxPixelPos));
}

CALL_SITE_DEBUGLOC
tsr_short2x2 AddAndClampPixelOffset(
	tsr_short2x2 KernelCenterPixelPos,
	tsr_short2x2 Offset,
	const tsr_short2 OffsetDirection,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	tsr_short2x2 SamplePixelPos = KernelCenterPixelPos + Offset;
	
	tsr_short2 SamplePixelPos0 = dpv_lo(SamplePixelPos);
	tsr_short2 SamplePixelPos1 = dpv_hi(SamplePixelPos);

	// Only do clamp based on the compile time known direction of the offset.
	// This turns only 1 v_pk_max_i16 and v_pk_min_i16 for a 3x3 kernel.
	if (OffsetDirection.x > 0)
		SamplePixelPos0.x = min(SamplePixelPos0.x, tsr_short(MaxPixelPos.x));
	else if (OffsetDirection.x < 0)
		SamplePixelPos0.x = max(SamplePixelPos0.x, tsr_short(MinPixelPos.x));
		
	if (OffsetDirection.y > 0)
		SamplePixelPos0.y = min(SamplePixelPos0.y, tsr_short(MaxPixelPos.y));
	else if (OffsetDirection.y < 0)
		SamplePixelPos0.y = max(SamplePixelPos0.y, tsr_short(MinPixelPos.y));
	
	if (OffsetDirection.x > 0)
		SamplePixelPos1.x = min(SamplePixelPos1.x, tsr_short(MaxPixelPos.x));
	else if (OffsetDirection.x < 0)
		SamplePixelPos1.x = max(SamplePixelPos1.x, tsr_short(MinPixelPos.x));
		
	if (OffsetDirection.y > 0)
		SamplePixelPos1.y = min(SamplePixelPos1.y, tsr_short(MaxPixelPos.y));
	else if (OffsetDirection.y < 0)
		SamplePixelPos1.y = max(SamplePixelPos1.y, tsr_short(MinPixelPos.y));
	
	return dpv_interleave_registers(SamplePixelPos0, SamplePixelPos1);
}

CALL_SITE_DEBUGLOC
tsr_short2 AddAndClampPixelOffset(
	tsr_short2 KernelCenterPixelPos,
	tsr_short2 Offset,
	const tsr_short2 OffsetDirection,
	int2 MinPixelPos, int2 MaxPixelPos)
{
	tsr_short2 SamplePixelPos = KernelCenterPixelPos + Offset;

	// Only do clamp based on the compile time known direction of the offset.
	// This turns only 1 v_pk_max_i16 and v_pk_min_i16 for a 3x3 kernel.
	if (OffsetDirection.x > 0)
		SamplePixelPos.x = min(SamplePixelPos.x, tsr_short(MaxPixelPos.x));
	else if (OffsetDirection.x < 0)
		SamplePixelPos.x = max(SamplePixelPos.x, tsr_short(MinPixelPos.x));
		
	if (OffsetDirection.y > 0)
		SamplePixelPos.y = min(SamplePixelPos.y, tsr_short(MaxPixelPos.y));
	else if (OffsetDirection.y < 0)
		SamplePixelPos.y = max(SamplePixelPos.y, tsr_short(MinPixelPos.y));

	return SamplePixelPos;
}


//------------------------------------------------------- 3x3 BUTTERFLY KERNEL

CALL_SITE_DEBUGLOC
tsr_short2 GetLaneOffsetSign()
#if CONFIG_BUTTERFLY_KERNEL
{
	uint LaneIndex = WaveGetLaneIndex();

	return tsr_short(-1) + tsr_short2(tsr_ushort2(tsr_ushort(LaneIndex) << tsr_ushort(1), LaneIndex) & tsr_ushort(0x2));
}
#else
{
	return tsr_short(1).xx;
}
#endif

CALL_SITE_DEBUGLOC
tsr_short2 GetNeighborhoodPixelPos(const uint ArrayIndex, tsr_short2 InputPixelPos)
{
	tsr_short2 LaneOffsetSign = GetLaneOffsetSign();

	tsr_short2 SampleInputPixelPos;
	#if CONFIG_BUTTERFLY_KERNEL
	{
		tsr_short2 Offset = tsr_short2(Offsets2x2[ArrayIndex]) * LaneOffsetSign;
		SampleInputPixelPos = ClampPixelOffset(InputPixelPos + Offset, InputPixelPosMin, InputPixelPosMax);
	}
	#else
	{
		tsr_short2 Offset = kOffsets3x3[ArrayIndex];
					
		SampleInputPixelPos = InputPixelPos + ClampPixelOffset(
			InputPixelPos,
			Offset, Offset,
			InputPixelPosMin, InputPixelPosMax);
	}
	#endif

	return SampleInputPixelPos;
}

CALL_SITE_DEBUGLOC
template<typename Type>
Type AccessNeighborhoodStaticOffset(Type Array[CONFIG_BUTTERFLY_SAMPLES], const int2 Offset)
{
	const uint OffsetIndex = uint(dot(Offset + 1, int2(1, 3)));

	#if CONFIG_BUTTERFLY_KERNEL
		const uint ButterflyArray[] = {
			0x3, 0x2, 0x2,
			0x1, 0x0, 0x0,
			0x1, 0x0, 0x0,
		};
		const uint IndexArray[] = {
			0, 0, 1,
			0, 0, 1,
			2, 2, 3,
		};

		const FWaveBroadcastSettings BroadcastSetting = InitWaveXorButterfly(ButterflyArray[OffsetIndex]);
		
		return WaveBroadcast(BroadcastSetting, Array[IndexArray[OffsetIndex]]);
	#else
		return Array[OffsetIndex];
	#endif
}

CALL_SITE_DEBUGLOC
template<typename Type>
Type AccessNeighborhoodDynamicOffset(Type Array[CONFIG_BUTTERFLY_SAMPLES], const tsr_short2 Offset)
{
	const uint OffsetIndex = uint(dot(Offset + 1, tsr_short2(1, 3)));

	#if CONFIG_BUTTERFLY_KERNEL
		const FWaveBroadcastSettings BroadcastVertical   = InitWaveXorButterfly(0x2);
		const FWaveBroadcastSettings BroadcastHorizontal = InitWaveXorButterfly(0x1);
		const FWaveBroadcastSettings BroadcastDiagonal   = InitWaveXorButterfly(0x3);

		Type Return = Array[0];
		Return = select(all(Offset == tsr_short2(-1, -1)), WaveBroadcast(BroadcastDiagonal,   Array[0]), Return);
		Return = select(all(Offset == tsr_short2( 0, -1)), WaveBroadcast(BroadcastVertical,   Array[0]), Return);
		Return = select(all(Offset == tsr_short2(+1, -1)), WaveBroadcast(BroadcastVertical,   Array[1]), Return);
		
		Return = select(all(Offset == tsr_short2(-1,  0)), WaveBroadcast(BroadcastHorizontal, Array[0]), Return);
		Return = select(all(Offset == tsr_short2(+1,  0)), Array[1], Return);
		
		Return = select(all(Offset == tsr_short2(-1, +1)), WaveBroadcast(BroadcastHorizontal, Array[2]), Return);
		Return = select(all(Offset == tsr_short2( 0, +1)), Array[2], Return);
		Return = select(all(Offset == tsr_short2(+1, +1)), Array[3], Return);
		
		return Return;
	#else
		return Array[OffsetIndex];
	#endif
}

CALL_SITE_DEBUGLOC
template<typename Type>
Type AccessNeighborhoodCenter(Type Array[CONFIG_BUTTERFLY_SAMPLES])
#if CONFIG_BUTTERFLY_KERNEL
{
	return Array[0];
}
#else
{
	return Array[4];
}
#endif


//------------------------------------------------------- OPTIMISED CATMULL-ROM

FCatmullRomSamples GetBicubic2DCatmullRomSamples_Stubbe(float2 UV, float2 Size, in float2 InvSize)
{
	float2 PixelCoord = UV * Size;

	float2 iuv = floor(PixelCoord - 0.5) + 1.0;
	tsr_half2 f = tsr_half2(PixelCoord - iuv);
	tsr_half2 f2 = f * f;
	tsr_half2 f_2 = f + f;

	tsr_half2 uv = (tsr_half(1.25) - f2) * f + tsr_half(0.5);
	tsr_half2 t0 = (tsr_half(0.25) * f2 - tsr_half(0.0625)) * (tsr_half(1.125) - tsr_half(0.5) * f2.yx);
	tsr_half2 w0 = t0 - f_2 * t0;
	tsr_half2 w3 = t0 + f_2 * t0;

	float2 cuv = (iuv - 0.5) * InvSize;
	uv *= tsr_half2(InvSize);

	FCatmullRomSamples Samples;
	Samples.Count = BICUBIC_CATMULL_ROM_SAMPLES;

	Samples.UV[0] = cuv + tsr_half2(               -InvSize.x,                      uv.y);
	Samples.UV[1] = cuv + tsr_half2(tsr_half(2.0) * InvSize.x,                      uv.y);
	Samples.UV[2] = cuv + uv;
	Samples.UV[3] = cuv + tsr_half2(                     uv.x,                -InvSize.y);
	Samples.UV[4] = cuv + tsr_half2(                     uv.x, tsr_half(2.0) * InvSize.y);
	
	Samples.Weight[0] = half(w0.x);
	Samples.Weight[1] = half(w3.x);
	Samples.Weight[2] = half(tsr_half(1.0) - w0.x - w3.x - w0.y - w3.y);
	Samples.Weight[3] = half(w0.y);
	Samples.Weight[4] = half(w3.y);

	Samples.UVDir[0] = int2(-1,  0);
	Samples.UVDir[1] = int2( 1,  0);
	Samples.UVDir[2] = int2( 0,  0);
	Samples.UVDir[3] = int2( 0, -1);
	Samples.UVDir[4] = int2( 0,  1);
	
	Samples.FinalMultiplier = 1.0;
	return Samples;
} // GetBicubic2DCatmullRomSamples_Stubbe()

template<typename Type>
struct FCatmullRomFetches
{
	Type Colors[BICUBIC_CATMULL_ROM_SAMPLES];
	tsr_half Weights[BICUBIC_CATMULL_ROM_SAMPLES];
	
	void FetchSamples(Texture2DArray<Type> InTexture, float2 UV, float SliceIndex, float2 Size, float2 InvSize, float2 MinUV, float2 MaxUV)
	{
		FCatmullRomSamples Samples = GetBicubic2DCatmullRomSamples_Stubbe(UV, Size, InvSize);
		
		UNROLL_N(BICUBIC_CATMULL_ROM_SAMPLES)
		for (uint i = 0; i < BICUBIC_CATMULL_ROM_SAMPLES; i++)
		{
			float2 SampleUV = Samples.UV[i];
			SampleUV = fastClamp(
				SampleUV,
				MinUV,
				MaxUV);

			Colors[i] = InTexture.SampleLevel(GlobalBilinearClampedSampler, float3(SampleUV, SliceIndex), 0);
			Weights[i] = tsr_half(Samples.Weight[i]);
		}
	}

	Type AccumulateSamples()
	{
		Type Color = tsr_half(0.0);
		UNROLL_N(BICUBIC_CATMULL_ROM_SAMPLES)
		for (uint i = 0; i < BICUBIC_CATMULL_ROM_SAMPLES; i++)
		{
			Color += Colors[i] * Weights[i];
		}
		return Color;
	}
};


//------------------------------------------------------- UPSCALING WEIGHT DISTRIBUTION

// Returns the weight of a pixels at a coordinate <PixelDelta> from the PDF highest point.
CALL_SITE_DEBUGLOC
tsr_half2 ComputeSampleWeigth(tsr_half2 UpscaleFactor, tsr_half2x2 PixelDelta, const float MinimalContribution)
{
	tsr_half2 u2 = UpscaleFactor * UpscaleFactor;

	// 1 - 1.9 * x^2 + 0.9 * x^4
	tsr_half2 x2 = saturate(u2 * dpv_length2(PixelDelta));
	//return tsr_half(((float(0.9) + MinimalContribution) * x2 - float(1.9)) * x2 + float(1.0));
	return saturate((tsr_half(0.9) * x2 - tsr_half(1.9)) * x2 + tsr_half(1.0 + MinimalContribution));
}

CALL_SITE_DEBUGLOC
tsr_half ComputeSampleWeigth(tsr_half UpscaleFactor, tsr_half2 PixelDelta, const float MinimalContribution)
{
	return dpv_lo(ComputeSampleWeigth(
		dpv_interleave_mono_registers(UpscaleFactor),
		dpv_interleave_mono_registers(PixelDelta),
		MinimalContribution));
}
