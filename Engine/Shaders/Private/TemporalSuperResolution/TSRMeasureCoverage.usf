// Copyright Epic Games, Inc. All Rights Reserved.


#include "../DeferredShadingCommon.ush"  
#include "TSRColorSpace.ush"
#include "TSRThinGeometryCommon.ush"
//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#define TILE_ROWS 2

#define THIN_GEOMETRY_SHADING_RANGE_FOLIAGE		 0
#define THIN_GEOMETRY_SHADING_RANGE_FOLIAGE_HAIR 1
#define THIN_GEOMETRY_SHADING_RANGE_ALL			 2

#define CONFIG_THIN_GEOMETRY_SHADING_RANGE (DIM_THIN_GEOMETRY_SHADING_RANGE)


//------------------------------------------------------- PARAMETERS

int OutputArrayIndex;
RWTexture2DArray<float> ThinGeometryCoverageOutput;

uint GetThinGeometry(const FGBufferData GBufferData)
{
#if CONFIG_THIN_GEOMETRY_SHADING_RANGE == THIN_GEOMETRY_SHADING_RANGE_ALL
	uint ThinGeometryFlag = 2u;
#else
	uint ThinGeometryFlag = ((GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE) 
	#if CONFIG_THIN_GEOMETRY_SHADING_RANGE == THIN_GEOMETRY_SHADING_RANGE_FOLIAGE_HAIR
		    ||(GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
	#endif
		    ) ? 2u : 1u;
#endif
	return select(GBufferData.ShadingModelID == SHADINGMODELID_UNLIT, 0u, ThinGeometryFlag);
}

//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void MeasureCoverageCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 GroupThreadId = uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);
	uint2 OutputPixelOrigin = (InputInfo_ViewportMin + GroupId * uint(TILE_SIZE * TILE_ROWS)) + GroupThreadId;

	uint ThinGeometryArray[TILE_ROWS * TILE_ROWS];

	ISOLATE
	{
		UNROLL_N(TILE_ROWS)
		for (uint y = 0; y < TILE_ROWS; y++)
		{
			UNROLL_N(TILE_ROWS)
			for (uint x = 0; x < TILE_ROWS; x++)
			{
				const uint2 PixelOffset = uint2(x, y) * TILE_SIZE;
				uint2 InputPixelPos = OutputPixelOrigin + PixelOffset;

				ThinGeometryArray[x + y * TILE_ROWS] = GetThinGeometry(GetGBufferDataUint(InputPixelPos));
			}
		}
	}

	ISOLATE
	{
		UNROLL_N(TILE_ROWS)
		for (uint y = 0; y < TILE_ROWS; y++)
		{
			UNROLL_N(TILE_ROWS)
			for (uint x = 0; x < TILE_ROWS; x++)
			{
				const uint2 PixelOffset = uint2(x, y) * TILE_SIZE;
				uint2 InputPixelPos = OutputPixelOrigin + PixelOffset;
				float ThinGeometry = EncodeThinGeometryMaterial(ThinGeometryArray[x + y * TILE_ROWS]);

				InputPixelPos.x = select(all(InputPixelPos < InputInfo_ViewportMax), InputPixelPos.x, uint(~0));
				ThinGeometryCoverageOutput[int3(InputPixelPos, OutputArrayIndex)] = ThinGeometry;
			}
		}
	}
}
