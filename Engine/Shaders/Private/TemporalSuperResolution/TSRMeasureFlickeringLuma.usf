// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRColorSpace.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#define TILE_ROWS 2


//------------------------------------------------------- PARAMETERS
float ExposureOffsetFactor;
Texture2D<tsr_half3> SceneColorTexture;
RWTexture2DArray<tsr_half> FlickeringLumaOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void MeasureFlickeringLumaCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint2 GroupThreadId = uint2(GroupThreadIndex % TILE_SIZE, GroupThreadIndex / TILE_SIZE);
	uint2 OutputPixelOrigin = (InputInfo_ViewportMin + GroupId * uint(TILE_SIZE * TILE_ROWS)) + GroupThreadId;

	tsr_half3 ColorArray[TILE_ROWS * TILE_ROWS];

	ISOLATE
	{
		UNROLL_N(TILE_ROWS)
		for (uint y = 0; y < TILE_ROWS; y++)
		{
			UNROLL_N(TILE_ROWS)
			for (uint x = 0; x < TILE_ROWS; x++)
			{
				const uint2 PixelOffset = uint2(x, y) * TILE_SIZE;
				uint2 InputPixelPos = OutputPixelOrigin + PixelOffset;
			
				ColorArray[x + y * TILE_ROWS] = SceneColorTexture[InputPixelPos];
			}
		}
	}

	ISOLATE
	{
		UNROLL_N(TILE_ROWS)
		for (uint y = 0; y < TILE_ROWS; y++)
		{
			UNROLL_N(TILE_ROWS)
			for (uint x = 0; x < TILE_ROWS; x++)
			{
				const uint2 PixelOffset = uint2(x, y) * TILE_SIZE;
				uint2 InputPixelPos = OutputPixelOrigin + PixelOffset;
			
				tsr_half3 LinearColor = ColorArray[x + y * TILE_ROWS] * tsr_half(ExposureOffsetFactor);
				tsr_half3 SMColor = LinearToSMCS(LinearColor);
				tsr_half SMLuma = dot(SMColor, kMoireSMCSWeights);
				tsr_half GLuma = SMCSToGCS(SMLuma.rrr).r;

				InputPixelPos.x = select(all(InputPixelPos < InputInfo_ViewportMax), InputPixelPos.x, uint(~0));

				FlickeringLumaOutput[int3(InputPixelPos, 0)] = GLuma;
			}
		}
	}
}
