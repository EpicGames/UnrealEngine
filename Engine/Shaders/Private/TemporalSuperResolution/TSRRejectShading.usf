// Copyright Epic Games, Inc. All Rights Reserved.

//------------------------------------------------------- INCLUDES

#define CONFIG_THIN_GEOMETRY_DETECTION (DIM_THIN_GEOMETRY_DETECTION)

#include "TSRShadingAnalysis.ush"
#include "TSRReprojectionField.ush"


//------------------------------------------------------- CONFIG

#define DEBUG_ARRAY_SIZE 8

#define DEBUG_DISABLE_PADDING 0

#define DEBUG_DIGESTABLE 0

/** Whether to detect flickering. */
#define CONFIG_MOIRE_DETECTION (DIM_FLICKERING_DETECTION)

#define CONFIG_HISTORY_RESURRECTION (DIM_HISTORY_RESURRECTION)


//------------------------------------------------------- CONSTANTS

#if CONFIG_LDS_DESPILL
	static const uint kDespillInput = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 0;
	static const uint kDespillHistory = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 1;
#endif


//------------------------------------------------------- PARAMETERS

FScreenTransform InputPixelPosToTranslucencyTextureUV;
float4x4 ClipToResurrectionClip;
float2 ResurrectionJacobianXMul;
float2 ResurrectionJacobianXAdd;
float2 ResurrectionJacobianYMul;
float2 ResurrectionJacobianYAdd;
float2 TranslucencyTextureUVMin;
float2 TranslucencyTextureUVMax;
float3 HistoryGuideQuantizationError;
float3 SceneColorOutputQuantizationError;
float TheoricBlendFactor;
uint TileOverscan;
uint bEnableResurrection;
uint bEnableFlickeringHeuristic;


Texture2D<tsr_halfC_resource> InputTexture;
Texture2D<tsr_half_resource>  InputMoireLumaTexture;
Texture2D<tsr_half4_resource> InputSceneTranslucencyTexture;
Texture2D<tsr_half4_resource> ReprojectedHistoryGuideTexture;
Texture2D<tsr_half_resource>  ReprojectedHistoryGuideMetadataTexture;
Texture2D<tsr_half4_resource> ReprojectedHistoryMoireTexture;
#if CONFIG_HISTORY_RESURRECTION
	Texture2D<tsr_half4_resource> ResurrectedHistoryGuideTexture;
	Texture2D<tsr_half_resource>  ResurrectedHistoryGuideMetadataTexture;
#endif
Texture2D<tsr_half2_resource> DecimateMaskTexture;
Texture2D<tsr_ushort_resource> IsMovingMaskTexture;
Texture2D<tsr_ushort_resource> ThinGeometryTexture;
Texture2D<float2> ClosestDepthTexture;

RWTexture2DArray<tsr_half4_resource> HistoryGuideOutput;
RWTexture2DArray<tsr_half4_resource> HistoryMoireOutput;
RWTexture2D<tsr_half4_resource>  HistoryRejectionOutput;
RWTexture2DArray<uint>  ReprojectionFieldOutput;
RWTexture2D<tsr_halfC_resource>  InputSceneColorOutput;
RWTexture2D<tsr_half_resource>   InputSceneColorLdrLumaOutput;
RWTexture2D<tsr_ushort_resource> AntiAliasMaskOutput;


//------------------------------------------------------- GLOBALS

static uint2 GGroupId = 0;

static tsr_tensor_half4 Debug[DEBUG_ARRAY_SIZE];


//------------------------------------------------------- INPUT FETCHES

tsr_short2 ComputeElementPixelPos(const uint ElementIndex)
{
	tsr_short2 ClampedFetchPixelPos = ComputeElementInputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);

	return ClampedFetchPixelPos;
}

tsr_tensor_short2 ComputePixelPos()
{
	tsr_tensor_short2 PixelPos;

	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		PixelPos.SetElement(ElementIndex, ComputeElementPixelPos(ElementIndex));
	}

	return PixelPos;
}

tsr_short2 ComputeElementOutputPixelPos(const uint ElementIndex)
{
	return ComputeElementOutputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);
}

void FetchThinGeometryEdge(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_half HistoryRelaxationWeight,
	out tsr_tensor_bool IsClusterHoleRegion)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		tsr_short EncodedThinGeometryWeight = tsr_short(ThinGeometryTexture[ClampedFetchPixelPos]);
		
		tsr_half Weight = tsr_half(EncodedThinGeometryWeight >> tsr_ushort(1)) * rcp(tsr_half(127.0));
		bool UseMultiplyOperation = (EncodedThinGeometryWeight & tsr_ushort(0x1))!= tsr_ushort(0x0);

		HistoryRelaxationWeight.SetElement(ElementIndex, Weight);
		IsClusterHoleRegion.SetElement(ElementIndex, UseMultiplyOperation);
	}
} // FetchMoireInput()

void FetchDecimateMask(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_half VelocityEdge,
	out tsr_tensor_bool bIsDisoccluded,
	out tsr_tensor_bool bHasPixelAnimation,
	out tsr_tensor_bool bIsResurrectionOffScreen)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		
		tsr_half2 DecimateMask = tsr_half2(DecimateMaskTexture[ClampedFetchPixelPos]);

		tsr_ushort BitMask = tsr_ushort(DecimateMask.r * tsr_half(255.0));

		VelocityEdge.SetElement(ElementIndex, DecimateMask.g);
		bIsDisoccluded.SetElement(ElementIndex,     (BitMask & tsr_ushort(0x23)) != tsr_ushort(0));
		bHasPixelAnimation.SetElement(ElementIndex, (BitMask & tsr_ushort(0x4)) != tsr_ushort(0));
		bIsResurrectionOffScreen.SetElement(ElementIndex, (BitMask & tsr_ushort(0x30)) != tsr_ushort(0));
	}
} // FetchDecimateMask()

void FetchSceneColorAndTranslucency(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_halfC OriginalOpaqueInput,
	out tsr_tensor_half4 OriginalTranslucencyInput)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		
		float2 TranslucencyTextureUV = ApplyScreenTransform(float2(ClampedFetchPixelPos), InputPixelPosToTranslucencyTextureUV);
		TranslucencyTextureUV = fastClamp(TranslucencyTextureUV, TranslucencyTextureUVMin, TranslucencyTextureUVMax);
		
		OriginalOpaqueInput.SetElement(ElementIndex, tsr_halfC(InputTexture[ClampedFetchPixelPos]));
		OriginalTranslucencyInput.SetElement(ElementIndex, tsr_half4(InputSceneTranslucencyTexture.SampleLevel(GlobalBilinearClampedSampler, TranslucencyTextureUV, 0)));
	}
} // FetchSceneColorAndTranslucency()

void FetchHistoryGuide(
	Texture2D<tsr_half4_resource> HistoryGuideTexture,
	Texture2D<tsr_half_resource>  HistoryGuideMetadataTexture,
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_halfC History,
	out tsr_tensor_half  HistoryUncertainty)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	#if CONFIG_SCENE_COLOR_ALPHA
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);

		History.SetElement(ElementIndex, tsr_half4(HistoryGuideTexture[ClampedFetchPixelPos]));
		HistoryUncertainty.SetElement(ElementIndex, tsr_half(HistoryGuideMetadataTexture[ClampedFetchPixelPos]));
	}
	#else
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);

		tsr_half4 ReprojectedGuide = tsr_half4(HistoryGuideTexture[ClampedFetchPixelPos]);
		History.SetElement(ElementIndex, ReprojectedGuide.rgb);
		HistoryUncertainty.SetElement(ElementIndex, ReprojectedGuide.a);
	}
	#endif
} // FetchHistoryGuide()

// Fetch the previous history history.
void FetchReprojectedHistoryGuide(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_halfC History,
	out tsr_tensor_half  HistoryUncertainty)
{
	FetchHistoryGuide(
		ReprojectedHistoryGuideTexture,
		ReprojectedHistoryGuideMetadataTexture,
		PixelPos,
		/* out */ History,
		/* out */ HistoryUncertainty);
}

#if CONFIG_MOIRE_DETECTION

void FetchMoireInput(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_half MoireInput,
	out tsr_tensor_half4 PrevMoireHistory,
	out tsr_tensor_half bIsStatic)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		
		MoireInput.SetElement(ElementIndex, tsr_half(InputMoireLumaTexture[ClampedFetchPixelPos]));
		PrevMoireHistory.SetElement(ElementIndex, tsr_half4(ReprojectedHistoryMoireTexture[ClampedFetchPixelPos]));
		bIsStatic.SetElement(ElementIndex, tsr_half(1.0) - tsr_half(IsMovingMaskTexture[ClampedFetchPixelPos]) * rcp(tsr_half(255.0)));
	}
} // FetchMoireInput()

#endif // CONFIG_MOIRE_DETECTION

#if CONFIG_HISTORY_RESURRECTION

// Fetch the resurrected history.
void FetchResurrectedHistoryGuide(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_halfC History,
	out tsr_tensor_half  HistoryUncertainty)
{
	FetchHistoryGuide(
		ResurrectedHistoryGuideTexture,
		ResurrectedHistoryGuideMetadataTexture,
		PixelPos,
		/* out */ History,
		/* out */ HistoryUncertainty);
}

// Returns whether resurrected frame is closer than previous frame on per pixel basis.
CALL_SITE_DEBUGLOC
tsr_tensor_bool IsResurrectedFrameCloserThanPreviousFrame(tsr_tensor_bool bIsResurrectionOffScreen, tsr_tensor_halfC ExposedInput, tsr_tensor_halfC ExposedHistory, tsr_tensor_halfC ExposedResurrectedHistory)
{
	tsr_tensor_halfC PrevMatch         = abs(ExposedInput - ExposedHistory);
	tsr_tensor_halfC ResurrectionMatch = abs(ExposedInput - ExposedResurrectedHistory);

	tsr_tensor_halfC MatchDiff = PrevMatch - ResurrectionMatch;
	#if CONFIG_SCENE_COLOR_ALPHA
		tsr_tensor_half TotalMatchDiff = Sum3x3(MatchDiff[0] + MatchDiff[1] + MatchDiff[2] + MatchDiff[3]);
	#else
		tsr_tensor_half TotalMatchDiff = Sum3x3(MatchDiff[0] + MatchDiff[1] + MatchDiff[2]);
	#endif

	tsr_tensor_half bResurrectedPixelIsCloser = saturate((TotalMatchDiff - tsr_tensor_half::Const(tsr_half(0.05 * 3.0 * 9.0))) * tsr_tensor_half::Const(tsr_half(POSITIVE_INFINITY)));

	tsr_tensor_bool bResurrectionIsCloser = and(Sum3x3(bResurrectedPixelIsCloser) > tsr_tensor_half::Const(4.0), not(bIsResurrectionOffScreen));

	//tsr_tensor_bool bResurrectionIsCloser = MedianBool3x3((TotalMatchDiff - tsr_tensor_half::Const(tsr_half(0.05 * 3.0 * 9.0))));

	return bResurrectionIsCloser;
}

// Whether it is worth for the entire tile to MeasureRejection() on the resurrected frame.
CALL_SITE_DEBUGLOC
bool IsWorthMeasuringRejectionOfResurrectedFrame(tsr_tensor_half PrevRejectionBlendFinal, tsr_tensor_bool bResurrectionIsCloser)
{
	bool bComputeResurrection;
	BRANCH
	if (bEnableResurrection)
	{
		tsr_tensor_bool bPrevIsNotGoodEnough = PrevRejectionBlendFinal < tsr_tensor_half::Const(0.5);
		tsr_tensor_bool bIsValidTilePixel = IsValidTilePixel<LANE_STRIDE_X, LANE_STRIDE_Y>(TileOverscan);
		tsr_tensor_bool bShouldComputeResurrection = and(and(bResurrectionIsCloser, bPrevIsNotGoodEnough), bIsValidTilePixel);

		bComputeResurrection = AnyThread(AnyComponent(bShouldComputeResurrection));
	}
	else
	{
		bComputeResurrection = false;
	}

	return bComputeResurrection;
}

// Final logic whether the resurrection must actually happen on per pixel basis.
CALL_SITE_DEBUGLOC
tsr_tensor_bool ShouldResurrectHistory(tsr_tensor_half PrevRejectionBlendFinal, tsr_tensor_half ResurrectionRejectionBlendFinal, tsr_tensor_bool bResurrectionIsCloser)
{
	tsr_tensor_bool bIsValidTilePixel = IsValidTilePixel<LANE_STRIDE_X, LANE_STRIDE_Y>(TileOverscan);
	tsr_tensor_bool bResurrectionIsBetter = (ResurrectionRejectionBlendFinal - PrevRejectionBlendFinal) > tsr_tensor_half::Const(0.1);
	tsr_tensor_bool bResurrectHistory = and(and(bResurrectionIsCloser, bResurrectionIsBetter), bIsValidTilePixel);

	return bResurrectHistory;
}

// Load the depth for ComputeResurrectionVelocity()
CALL_SITE_DEBUGLOC
void FetchDeviceZ(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_float DeviceZ)
{
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		
		DeviceZ.SetElement(ElementIndex, ClosestDepthTexture[ClampedFetchPixelPos].g);
	}
} // FetchMoireInput()

// Overwrite the reprojection field for pixels that needs resurrection.
CALL_SITE_DEBUGLOC
void OverwriteReprojectionFieldWithResurrection(tsr_tensor_bool bResurrectHistory, tsr_tensor_float DeviceZ)
{
	uint EncodedReprojectionVector[SIMD_SIZE];
	uint EncodedReprojectionJacobian[SIMD_SIZE];

	PLATFORM_SPECIFIC_ISOLATE
	{
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		ISOLATE
		{
			float LocalDeviceZ = DeviceZ.GetElement(ElementIndex);
		
			tsr_short2 InputPixelPos = ComputeElementPixelPos(ElementIndex);
			float2 ScreenPos = ApplyScreenTransform(float2(InputPixelPos), InputPixelPosToScreenPos);
			
			float4 ThisClip = float4(ScreenPos, DeviceZ.GetElement(ElementIndex), 1);
			float4 ResurrectionClip = mul(ThisClip, ClipToResurrectionClip);
			float  ResurrectionProjection = rcp(ResurrectionClip.w);
			float2 ResurrectionReprojectionVector = ScreenPos - ResurrectionClip.xy * ResurrectionProjection;
			EncodedReprojectionVector[ElementIndex] = EncodeReprojectionVector(ResurrectionReprojectionVector);

			// Compute the Jacobian from the PrevClip. Should technically take into account the depth delta along X and Y, but it's costly to compute, and it's using the dilated DeviceZ anyway, and not even using a 2x3 jacobian.
			tsr_half2x2 ResurrectionJacobian;
			ResurrectionJacobian[0] = tsr_half2(ResurrectionJacobianXMul) * tsr_half(ResurrectionProjection) + tsr_half2(ResurrectionJacobianXAdd);
			ResurrectionJacobian[1] = tsr_half2(ResurrectionJacobianYMul) * tsr_half(ResurrectionProjection) + tsr_half2(ResurrectionJacobianYAdd);

			EncodedReprojectionJacobian[ElementIndex] = EncodeReprojectionJacobian(ResurrectionJacobian);
		}
	}

	PLATFORM_SPECIFIC_ISOLATE
	{
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
			OutputPixelPos.x = select(bResurrectHistory.GetElement(ElementIndex), OutputPixelPos.x, tsr_short(-1));

			ReprojectionFieldOutput[tsr_short3(OutputPixelPos, kReprojectionVectorOutputIndex)]   = EncodedReprojectionVector[ElementIndex];
			ReprojectionFieldOutput[tsr_short3(OutputPixelPos, kReprojectionJacobianOutputIndex)] = EncodedReprojectionJacobian[ElementIndex];
		}
	}
} // OverwriteReprojectionFieldWithResurrection()

#endif // CONFIG_HISTORY_RESURRECTION


//------------------------------------------------------- UPDATE GUDIE

// Update the guide for next frame.
CALL_SITE_DEBUGLOC
tsr_tensor_halfC UpdateGuide(tsr_tensor_halfC ExposedInput, tsr_tensor_halfC ExposedHistory, tsr_tensor_bool bIsDisoccluded, tsr_tensor_half RejectionBlendFinal)
{
	tsr_tensor_bool bIsOffScreen = tsr_tensor_bool::Const(false);

	tsr_tensor_half TotalBlend = select(or(bIsOffScreen, bIsDisoccluded), tsr_tensor_half::Const(1.0), saturate(tsr_tensor_half::Const(1.0) - RejectionBlendFinal * tsr_tensor_half::Const(4.0)));
	tsr_tensor_half RejectedBlendFactor = max(tsr_tensor_half::Const(tsr_half(TheoricBlendFactor)), tsr_tensor_half::Const(1.0) - RejectionBlendFinal);

	tsr_tensor_half BlendFinal = max(RejectedBlendFactor, TotalBlend);

	tsr_tensor_halfC FinalGuide = ExposedInput * tsr_tensor_halfC::Vectorize(BlendFinal) - ExposedHistory * tsr_tensor_halfC::Vectorize(BlendFinal - 1.0);
	return FinalGuide;
}


//------------------------------------------------------- REGISTER PRESSURE HELPERS

#if CONFIG_LDS_DESPILL || CONFIG_CACHE_REFETCH

void RefetchInputAndHistoryFromLDSOrCache(
	out tsr_tensor_halfC Input,
	out tsr_tensor_half  InputUncertainty,
	out tsr_tensor_halfC History,
	out tsr_tensor_half  HistoryUncertainty)
#if CONFIG_LDS_DESPILL
{
	Deconcatenate(tsr_tensor_half4::LoadLDSSpill(kDespillInput), /* out */ Input, /* out */ InputUncertainty);
	Deconcatenate(tsr_tensor_half4::LoadLDSSpill(kDespillHistory), /* out */ History, /* out */ HistoryUncertainty);
}
#else
{
	tsr_tensor_short2 PixelPos = ComputePixelPos();

	tsr_tensor_halfC OriginalOpaqueInput;
	tsr_tensor_half4 OriginalTranslucencyInput;
	tsr_tensor_halfC OriginalHistory;
	tsr_tensor_half  OriginalHistoryUncertainty;
	tsr_tensor_half  VelocityEdge;
	tsr_tensor_bool  bIsDisoccluded;
	tsr_tensor_bool  bHasPixelAnimation;
	tsr_tensor_bool  bIsResurrectionOffScreen;

	FetchDecimateMask(PixelPos, /* out */ VelocityEdge, /* out */ bIsDisoccluded, /* out */ bHasPixelAnimation, /* out */ bIsResurrectionOffScreen);
	FetchSceneColorAndTranslucency(PixelPos, /* out */ OriginalOpaqueInput, /* out */ OriginalTranslucencyInput);
	FetchReprojectedHistoryGuide(
		PixelPos,
		/* out */ OriginalHistory,
		/* out */ OriginalHistoryUncertainty);

	tsr_tensor_halfC OriginalInput = ComposeTranslucencyForRejection(OriginalOpaqueInput, OriginalTranslucencyInput, bHasPixelAnimation);

	Input            = OriginalInput;
	InputUncertainty = VelocityEdge;
	History            = OriginalHistory;
	HistoryUncertainty = OriginalHistoryUncertainty;
}
#endif

#endif // CONFIG_LDS_DESPILL || CONFIG_CACHE_REFETCH


//------------------------------------------------------- ENTRY POINT

#if COMPILER_SUPPORTS_WAVE_SIZE && DIM_WAVE_SIZE > 0
WAVESIZE(DIM_WAVE_SIZE)
#endif
[numthreads(LANE_COUNT * WAVE_COUNT, 1, 1)]
void RejectShadingCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	GGroupId = GroupId;
	GGroupThreadIndex = GroupThreadIndex;

	UNROLL_N(DEBUG_ARRAY_SIZE)
	for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
	{
		Debug[DebugId].SetAllElements(0.0);
	}
	#if CONFIG_THIN_GEOMETRY_DETECTION
	bool bApplyThinGeometryRelaxation = false;
	#endif	
	// Analyses the moire pattern overtime.
	tsr_tensor_half  MoireError;
	#if CONFIG_MOIRE_DETECTION
	BRANCH
	if (bEnableFlickeringHeuristic)
	PLATFORM_SPECIFIC_ISOLATE
	{
		// Load its own data
		tsr_tensor_half  VelocityEdge;
		tsr_tensor_bool  bIsDisoccluded;
		tsr_tensor_bool  bHasPixelAnimation;
		tsr_tensor_bool  bIsResurrectionOffScreen;
		tsr_tensor_halfC OriginalOpaqueInput;
		tsr_tensor_half4 OriginalTranslucencyInput;
		tsr_tensor_half  MoireInput;
		tsr_tensor_half4 PrevMoireHistory;
		tsr_tensor_half  bIsStatic;
		PLATFORM_SPECIFIC_ISOLATE
		{
			tsr_tensor_short2 PixelPos = ComputePixelPos();
		
			FetchDecimateMask(PixelPos, /* out */ VelocityEdge, /* out */ bIsDisoccluded, /* out */ bHasPixelAnimation, /* out */ bIsResurrectionOffScreen);
			FetchSceneColorAndTranslucency(PixelPos, /* out */ OriginalOpaqueInput, /* out */ OriginalTranslucencyInput);
			FetchMoireInput(
				PixelPos,
				/* out */ MoireInput,
				/* out */ PrevMoireHistory,
				/* out */ bIsStatic);
		}
		
		#if CONFIG_THIN_GEOMETRY_DETECTION
		tsr_tensor_half ThinGeometryRelaxationFactor = tsr_tensor_half::Const(0);
		if (ThinGeometryAffectMoireError)
		{
			tsr_tensor_short2 PixelPos = ComputePixelPos();
			tsr_tensor_bool IsThinGeometryRegion;
			FetchThinGeometryEdge(PixelPos, ThinGeometryRelaxationFactor, IsThinGeometryRegion);
			bApplyThinGeometryRelaxation = AnyThread(AnyComponent(ThinGeometryRelaxationFactor > tsr_tensor_half::Const(0)));
		}
		#endif

		tsr_tensor_half4 MoireHistory;
		ComputeMoireError(
			bIsDisoccluded,
			bIsStatic,
			MoireInput,
			OriginalOpaqueInput,
			OriginalTranslucencyInput,
			PrevMoireHistory,
		#if CONFIG_THIN_GEOMETRY_DETECTION
			ThinGeometryRelaxationFactor,
			bApplyThinGeometryRelaxation,
		#endif
			/* out */ MoireHistory,
			/* out */ MoireError);

		// Stores moire output to save VGPR for what is next.
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
			const tsr_short2 OutputHistoryPixelPos = OutputPixelPos - tsr_short2(InputInfo_ViewportMin);

			HistoryMoireOutput[tsr_short3(OutputHistoryPixelPos, 0)] = MoireHistory.GetElement(ElementIndex);
		}
	}
	else
	#endif // CONFIG_MOIRE_DETECTION
	{
		MoireError = tsr_tensor_half::Const(0.0);
	}
	
	// Load all the history data.
	tsr_tensor_half  VelocityEdge;
	tsr_tensor_bool  bIsDisoccluded;
	tsr_tensor_bool  bHasPixelAnimation;
	tsr_tensor_bool  bIsResurrectionOffScreen;
	tsr_tensor_halfC OriginalOpaqueInput;
	tsr_tensor_half4 OriginalTranslucencyInput;
	tsr_tensor_halfC OriginalHistory;
	tsr_tensor_half  OriginalHistoryUncertainty;
	#if CONFIG_HISTORY_RESURRECTION
		tsr_tensor_halfC OriginalResurrectedHistory;
		tsr_tensor_half  OriginalResurrectedHistoryUncertainty;
	#endif
	{
		#if CONFIG_HISTORY_RESURRECTION
		BRANCH
		if (bEnableResurrection)
		PLATFORM_SPECIFIC_ISOLATE
		{
			tsr_tensor_short2 PixelPos = ComputePixelPos();

			FetchDecimateMask(PixelPos, /* out */ VelocityEdge, /* out */ bIsDisoccluded, /* out */ bHasPixelAnimation, /* out */ bIsResurrectionOffScreen);
			FetchSceneColorAndTranslucency(PixelPos, /* out */ OriginalOpaqueInput, /* out */ OriginalTranslucencyInput);
			FetchReprojectedHistoryGuide(
				PixelPos,
				/* out */ OriginalHistory,
				/* out */ OriginalHistoryUncertainty);
			FetchResurrectedHistoryGuide(
				PixelPos,
				/* out */ OriginalResurrectedHistory,
				/* out */ OriginalResurrectedHistoryUncertainty);
		}
		else // if (!bEnableResurrection)
		#endif // CONFIG_HISTORY_RESURRECTION
		PLATFORM_SPECIFIC_ISOLATE
		{
			tsr_tensor_short2 PixelPos = ComputePixelPos();

			FetchDecimateMask(PixelPos, /* out */ VelocityEdge, /* out */ bIsDisoccluded, /* out */ bHasPixelAnimation, /* out */ bIsResurrectionOffScreen);
			FetchSceneColorAndTranslucency(PixelPos, /* out */ OriginalOpaqueInput, /* out */ OriginalTranslucencyInput);
			FetchReprojectedHistoryGuide(
				PixelPos,
				/* out */ OriginalHistory,
				/* out */ OriginalHistoryUncertainty);
			#if CONFIG_HISTORY_RESURRECTION
				OriginalResurrectedHistory            = tsr_tensor_halfC::Const(0.0);
				OriginalResurrectedHistoryUncertainty = tsr_tensor_half::Const(0.0);
			#endif
		}
	}
	
	// Despill to LDS to make room in VGPRs for the outputing of InputSceneColorOutput
	#if CONFIG_LDS_DESPILL
	PLATFORM_SPECIFIC_ISOLATE
	{
		Concatenate(OriginalHistory, OriginalHistoryUncertainty).StoreLDSSpill(kDespillHistory);
	}
	#endif // CONFIG_LDS_DESPILL

	// Compute AA Luma and compose separate translucency into scene color for optimised HistoryUpdate.
	{
		tsr_tensor_halfC CenterFinalSceneColor = ComposeTranslucency(OriginalOpaqueInput, OriginalTranslucencyInput);
		CenterFinalSceneColor = min(CenterFinalSceneColor, tsr_tensor_halfC::Const(tsr_half(Max10BitsFloat)));
		
		tsr_tensor_half CenterLumaLDR = ComputeSpatialAntiAliaserLumaLDR(CenterFinalSceneColor);

		PLATFORM_SPECIFIC_ISOLATE
		{
			UNROLL_N(SIMD_SIZE)
			for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
			{
				const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
				
			#if CONFIG_ENABLE_STOCASTIC_QUANTIZATION
				// Banding removal due to loss of precision when saving 16 bit float data into 11_11_10 float.
				// Adding dithering based on mantissa bits precision stored in SceneColorOutputQuantizationError
				uint Random = Rand3DPCG16(int3(OutputPixelPos.xy, View.StateFrameIndexMod8)).x;
				tsr_half E = tsr_half(Rand16ToFloat(Random));
				tsr_halfC DitheredColor = CenterFinalSceneColor.GetElement(ElementIndex);
				DitheredColor.xyz = (tsr_half3)QuantizeForFloatRenderTarget(DitheredColor.xyz, E, SceneColorOutputQuantizationError);

				InputSceneColorOutput[OutputPixelPos] = DitheredColor;
			#else
				InputSceneColorOutput[OutputPixelPos] = CenterFinalSceneColor.GetElement(ElementIndex);
			#endif
				
				InputSceneColorLdrLumaOutput[OutputPixelPos] = CenterLumaLDR.GetElement(ElementIndex);
			}
		}
	}

	tsr_tensor_halfC OriginalInput = ComposeTranslucencyForRejection(OriginalOpaqueInput, OriginalTranslucencyInput, bHasPixelAnimation);
	
	// Despill to LDS to make room in VGPRs for the convolutions of MeasureRejection()
	#if CONFIG_LDS_DESPILL
	PLATFORM_SPECIFIC_ISOLATE
	{
		Concatenate(OriginalInput, VelocityEdge).StoreLDSSpill(kDespillInput);
	}
	#endif // CONFIG_LDS_DESPILL

	// Measure rejection
	tsr_tensor_half RejectionBlendFinal = tsr_tensor_half::Const(1.0);
	tsr_tensor_half RejectionClampBlend = tsr_tensor_half::Const(1.0);
	tsr_tensor_bool bAliasingIsVisible = tsr_tensor_bool::Const(false);
#if CONFIG_THIN_GEOMETRY_DETECTION
	tsr_tensor_half ThinGeometryRelaxationFactor = tsr_tensor_half::Const(0);
	{
		tsr_tensor_short2 PixelPos = ComputePixelPos();
		tsr_tensor_bool IsThinGeometryRegion;
		#if CONFIG_MOIRE_DETECTION
			BRANCH
			if (bApplyThinGeometryRelaxation || !ThinGeometryAffectMoireError || !bEnableFlickeringHeuristic)
			{
				FetchThinGeometryEdge(PixelPos, ThinGeometryRelaxationFactor, IsThinGeometryRegion);
			}
	
			BRANCH
			if (!ThinGeometryAffectMoireError || !bEnableFlickeringHeuristic)
			{
				bApplyThinGeometryRelaxation = AnyThread(AnyComponent(ThinGeometryRelaxationFactor > tsr_tensor_half::Const(0)));
			}
		#else
			FetchThinGeometryEdge(PixelPos, ThinGeometryRelaxationFactor, IsThinGeometryRegion);
			bApplyThinGeometryRelaxation = AnyThread(AnyComponent(ThinGeometryRelaxationFactor > tsr_tensor_half::Const(0)));
		#endif
	}
#endif
	#if CONFIG_HISTORY_RESURRECTION
		tsr_tensor_bool bResurrectionIsCloser;
	#endif
	PLATFORM_SPECIFIC_ISOLATE
	{
		// Reload all the data from LDS or cache with moire to avoid register pressure
		tsr_tensor_halfC Input = OriginalInput;
		tsr_tensor_halfC History;
		#if CONFIG_LDS_DESPILL
			tsr_tensor_half HistoryUncertainty;
			Deconcatenate(tsr_tensor_half4::LoadLDSSpill(kDespillHistory), /* out */ History, /* out */ HistoryUncertainty);
		#else
			History = OriginalHistory;
		#endif

		// Convert input to shading LDR
		tsr_tensor_halfC ExposedInput = LinearToSMCS(Input * tsr_half(ExposureOffsetFactor));
		tsr_tensor_halfC ExposedHistory = GCSToSMCS(History);
			
		// Check whether the history resurrection is closer to input, to discard and no pay the register pressure of OriginalResurrectedHistory
		#if CONFIG_HISTORY_RESURRECTION
		BRANCH
		if (bEnableResurrection)
		{
			tsr_tensor_halfC ExposedResurrectedHistory = GCSToSMCS(OriginalResurrectedHistory);
				
			bResurrectionIsCloser = IsResurrectedFrameCloserThanPreviousFrame(bIsResurrectionOffScreen, ExposedInput, ExposedHistory, ExposedResurrectedHistory);
		}
		else
		{
			bResurrectionIsCloser = tsr_tensor_bool::Const(false);
		}
		#endif
		
		// Mutual Annihilation
		tsr_tensor_halfC ExposedInputMin;
		tsr_tensor_halfC ExposedInputMax;
		MinMax3x3(ExposedInput, /* out */ ExposedInputMin, /* out */ ExposedInputMax);

		bAliasingIsVisible = IsAliasingVisible(ExposedInputMax - ExposedInputMin);

		tsr_tensor_halfC ClampedInput = Clamp3x3(ExposedInput, fastClamp(ExposedHistory, ExposedInputMin, ExposedInputMax));
		tsr_tensor_halfC ClampedHistory = AnnihilateToGuide3x3(ExposedHistory, /* Guide = */ ExposedInput);

		MeasureRejection(
			/* InputC0 = */    ExposedInput,
			/* InputC2 = */    ClampedInput,
			/* HistoryC2 = */  ClampedHistory,
			/* MoireError = */ MoireError,
#if CONFIG_THIN_GEOMETRY_DETECTION
			ThinGeometryRelaxationFactor,
			bApplyThinGeometryRelaxation,
#endif
			/* bEnableMoireError = */ CONFIG_MOIRE_DETECTION != 0 && bEnableFlickeringHeuristic != 0,
			/* out */ RejectionBlendFinal,
			/* out */ RejectionClampBlend);
	}
	
	// Try resurrecting the history if necessary
	tsr_tensor_bool bResurrectHistory = tsr_tensor_bool::Const(false);
	#if CONFIG_HISTORY_RESURRECTION
	bool bComputeResurrection = IsWorthMeasuringRejectionOfResurrectedFrame(RejectionBlendFinal, bResurrectionIsCloser);
	BRANCH
	if (bComputeResurrection)
	PLATFORM_SPECIFIC_ISOLATE
	{
		// Reload the resurrected history from cache.
		tsr_tensor_halfC Input;
		tsr_tensor_halfC ResurrectedHistory;
		tsr_tensor_half  ResurrectedHistoryUncertainty;
		#if CONFIG_CACHE_REFETCH || CONFIG_LDS_DESPILL
		{
			tsr_tensor_short2 PixelPos = ComputePixelPos();
		
			FetchResurrectedHistoryGuide(
				PixelPos,
				/* out */ ResurrectedHistory,
				/* out */ ResurrectedHistoryUncertainty);
				
			#if CONFIG_LDS_DESPILL
				Input = ResizeChannels<3>(tsr_tensor_half4::LoadLDSSpill(kDespillInput));
			#else
				tsr_tensor_halfC OriginalOpaqueInput;
				tsr_tensor_half4 OriginalTranslucencyInput;
				FetchSceneColorAndTranslucency(PixelPos, /* out */ OriginalOpaqueInput, /* out */ OriginalTranslucencyInput);

				Input = ComposeTranslucencyForRejection(OriginalOpaqueInput, OriginalTranslucencyInput, bHasPixelAnimation);
			#endif
		}
		#else
		ResurrectedHistory = OriginalResurrectedHistory;
		Input = OriginalInput;
		#endif
		
		// Convert input to shading LDR
		tsr_tensor_halfC ExposedInput = LinearToSMCS(Input * tsr_half(ExposureOffsetFactor));
		tsr_tensor_halfC ExposedResurrectedHistory = GCSToSMCS(ResurrectedHistory);
			
		// Mutual Annihilation
		tsr_tensor_halfC ClampedResurrectedInput, ClampedResurrectedHistory;
		AnnihilateMutually3x3(ExposedInput, ExposedResurrectedHistory, /* out */ ClampedResurrectedInput, /* out */ ClampedResurrectedHistory);

		tsr_tensor_half ResurrectionRejectionBlendFinal = tsr_tensor_half::Const(1.0);
		tsr_tensor_half ResurrectionRejectionClampBlend = tsr_tensor_half::Const(1.0);
		MeasureRejection(
			/* InputC0 = */    ExposedInput,
			/* InputC2 = */    ClampedResurrectedInput,
			/* HistoryC2 = */  ClampedResurrectedHistory,
			/* MoireError = */ tsr_tensor_half::Const(0.0),
#if CONFIG_THIN_GEOMETRY_DETECTION
			/*ThinGeometryRelaxationFactor */tsr_tensor_half::Const(0),
			false,
#endif
			/* bEnableMoireError = */ CONFIG_MOIRE_DETECTION != 0 && bEnableFlickeringHeuristic != 0,
			/* out */ ResurrectionRejectionBlendFinal,
			/* out */ ResurrectionRejectionClampBlend);
			
		bResurrectHistory = ShouldResurrectHistory(RejectionBlendFinal, ResurrectionRejectionBlendFinal, bResurrectionIsCloser);

		RejectionBlendFinal = select(bResurrectHistory, ResurrectionRejectionBlendFinal, RejectionBlendFinal);
		RejectionClampBlend = select(bResurrectHistory, ResurrectionRejectionClampBlend, RejectionClampBlend);
	} // if (bComputeResurrection)
	#endif // CONFIG_HISTORY_RESURRECTION
	
	// Update the guide for next frame.
	tsr_tensor_halfC FinalGuide;
	tsr_tensor_half  InputUncertainty;
	tsr_tensor_half  HistoryUncertainty;
	PLATFORM_SPECIFIC_ISOLATE
	{
		// Reload all the data from LDS or cache
		tsr_tensor_halfC Input;
		tsr_tensor_halfC History;
		#if CONFIG_LDS_DESPILL || CONFIG_CACHE_REFETCH
			RefetchInputAndHistoryFromLDSOrCache(
				/* out */ Input,
				/* out */ InputUncertainty,
				/* out */ History,
				/* out */ HistoryUncertainty);
		#else
			Input            = OriginalInput;
			InputUncertainty = VelocityEdge;
			History            = OriginalHistory;
			HistoryUncertainty = OriginalHistoryUncertainty;
		#endif
	
		// Apply resurrection
		#if CONFIG_HISTORY_RESURRECTION
		BRANCH
		if (AnyComponent(bResurrectHistory))
		PLATFORM_SPECIFIC_ISOLATE
		{
			// Reload the resurrected history from cache.
			tsr_tensor_halfC ResurrectedHistory;
			tsr_tensor_half  ResurrectedHistoryUncertainty;
			tsr_tensor_float DeviceZ;
			PLATFORM_SPECIFIC_ISOLATE
			{
				tsr_tensor_short2 PixePos = ComputePixelPos();
				
				#if CONFIG_CACHE_REFETCH
					FetchResurrectedHistoryGuide(
						PixePos,
						/* out */ ResurrectedHistory,
						/* out */ ResurrectedHistoryUncertainty);
				#else
					ResurrectedHistory            = OriginalResurrectedHistory;
					ResurrectedHistoryUncertainty = OriginalResurrectedHistoryUncertainty;
				#endif
					
				FetchDeviceZ(PixePos, /* out */ DeviceZ);
			}

			// Apply the resurrection on the guide immediatly to free up registers.
			History = select(tsr_tensor_boolC::Vectorize(bResurrectHistory), ResurrectedHistory, History);
			HistoryUncertainty = select(bResurrectHistory, ResurrectedHistoryUncertainty, HistoryUncertainty);
			
			// Apply the resurrection the reprojection field.
			OverwriteReprojectionFieldWithResurrection(bResurrectHistory, DeviceZ);
		}
		#endif // CONFIG_HISTORY_RESURRECTION

		// Convert input to same color space as the accumulation
		tsr_tensor_halfC ExposedInput   = AddPerceptionAdd(Input * tsr_half(ExposureOffsetFactor),                /* DestPerceptionAdd = */ kHistoryAccumulationPreceptionAdd);
		tsr_tensor_halfC ExposedHistory = AddPerceptionAdd(GCSToLinear(History), /* DestPerceptionAdd = */ kHistoryAccumulationPreceptionAdd);
		
		tsr_tensor_halfC ExposedFinalGuide = UpdateGuide(ExposedInput, ExposedHistory, bIsDisoccluded, RejectionBlendFinal);

		FinalGuide = LinearToGCS(RemovePerceptionAdd(ExposedFinalGuide, /* SourcePerceptionAdd = */ kHistoryAccumulationPreceptionAdd));
	}

	// Controls how the history and its validity is clamped.
	tsr_tensor_half  DisableHistoryClamp;
	tsr_tensor_half  IncreaseValidityMultiplier;
	PLATFORM_SPECIFIC_ISOLATE
	{
		DisableHistoryClamp        = select(and(bIsDisoccluded, not(bResurrectHistory)), tsr_tensor_half::Const(0.0), min(RejectionClampBlend, min(InputUncertainty, Min3x3(HistoryUncertainty))));
		IncreaseValidityMultiplier = select(and(bIsDisoccluded, not(bResurrectHistory)), tsr_tensor_half::Const(0.0), RejectionClampBlend);
	}
	
	// Whether to run spatial anti-aliaser on the frame. Must match SpatialAntiAliasingLerp.
	tsr_tensor_bool bSpatialAntiAlias = ShouldSpatialAntiAlias(bIsDisoccluded, bResurrectHistory, bAliasingIsVisible, RejectionBlendFinal);

	// Output data.
	PLATFORM_SPECIFIC_ISOLATE
	{
		const uint LaneIndex = GGroupThreadIndex;

		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			const tsr_short2 LaneSimdPixelOffset = GetElementPixelOffsetInTile(LANE_STRIDE_X, LANE_STRIDE_Y, GGroupThreadIndex, ElementIndex);
			const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
			const tsr_short2 OutputHistoryPixelPos = OutputPixelPos - tsr_short2(InputInfo_ViewportMin);

			tsr_halfC FinalGuideColor = FinalGuide.GetElement(ElementIndex);
			
			tsr_half BitMask = (
				select(bIsDisoccluded.GetElement(ElementIndex),    tsr_half(0x0), tsr_half(0x1) * rcp(tsr_half(255.0))) + 
				select(bResurrectHistory.GetElement(ElementIndex), tsr_half(0x2) * rcp(tsr_half(255.0)), tsr_half(0x0)));

			#if CONFIG_ENABLE_STOCASTIC_QUANTIZATION
			{
				uint Random = Rand3DPCG16(int3(LaneSimdPixelOffset, View.StateFrameIndexMod8)).x;
				tsr_half E = tsr_half(Rand16ToFloat(Random));
		
				FinalGuideColor.rgb = QuantizeForUNormRenderTarget(FinalGuideColor.rgb, E.x, HistoryGuideQuantizationError);
			}
			#endif

			#if CONFIG_SCENE_COLOR_ALPHA
				HistoryGuideOutput[tsr_short3(OutputHistoryPixelPos, 0)] = FinalGuideColor;
				HistoryGuideOutput[tsr_short3(OutputHistoryPixelPos, 1)] = tsr_half4(InputUncertainty.GetElement(ElementIndex), 0, 0, 0);
			#else
				HistoryGuideOutput[tsr_short3(OutputHistoryPixelPos, 0)] = tsr_half4(FinalGuideColor.xyz, InputUncertainty.GetElement(ElementIndex));
			#endif
			{
				const tsr_half kHalfOverTwoFiftySix = tsr_half(0.5) * tsr_half(rcp(256.0));
				const tsr_half k255HalfOverTwoFiftySix = tsr_half(255.5) * tsr_half(rcp(256.0));
				HistoryRejectionOutput[OutputPixelPos] = tsr_half4(
					RejectionBlendFinal.GetElement(ElementIndex) - kHalfOverTwoFiftySix,
					DisableHistoryClamp.GetElement(ElementIndex) - kHalfOverTwoFiftySix,
					k255HalfOverTwoFiftySix - IncreaseValidityMultiplier.GetElement(ElementIndex),
					BitMask);
			}
			AntiAliasMaskOutput[OutputPixelPos] = select(bSpatialAntiAlias.GetElement(ElementIndex), tsr_ushort(1), tsr_ushort(0));
		}
	}
	
	#if DEBUG_OUTPUT
	{
		// Mark the top left corner of the tile for debugging purposes.
		#if DEBUG_DISABLE_PADDING == 2
		if (GGroupThreadIndex == 0)
		{
			UNROLL_N(DEBUG_ARRAY_SIZE)
			for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
			{
				Debug[DebugId].SetElement(/* ElementIndex = */ 0, 1.0);
			}
		}
		#endif // DEBUG_DISABLE_PADDING == 2
	
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(
				/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
				TileOverscan,
				InputInfo_ViewportMin,
				InputInfo_ViewportMax,
				GGroupId,
				GGroupThreadIndex,
				ElementIndex,
				/* OutputDataOffset = */ int2(0, 0),
				/* OutputResDivisor = */ 1,
				/* bDebugDisablePadding = */ DEBUG_DISABLE_PADDING != 0);

			UNROLL_N(DEBUG_ARRAY_SIZE)
			for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
			{
				DebugOutput[tsr_short3(OutputPixelPos, DebugId)] = float4(Debug[DebugId].GetElement(ElementIndex));
			}
		}
	}
	#endif // DEBUG_OUTPUT
}
