// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRCommon.ush"


//------------------------------------------------------- CONSTANTS

static const uint kReprojectionVectorOutputIndex = 0;
static const uint kReprojectionJacobianOutputIndex = 1;
static const uint kReprojectionBoundaryOutputIndex = 2;


//------------------------------------------------------- REPROJECTION's DILATING OFFSET & BOUNDARY

static const tsr_half2 kFullDilateBoundary = tsr_half2(0.0, 1.0);

uint EncodeReprojectionOffset(tsr_short2 ReprojectionOffset)
{
	return  uint(dot(tsr_ushort2(ReprojectionOffset + tsr_ushort(1)), tsr_ushort2(1, 4)));
}

/** Encodes the reprojection field's dilating offset [-1;1]^2 in first 4bits, and ReprojectionBoundary.xy in 11bit eachs, 26bits total. */
uint EncodeReprojectionBoundary(tsr_short2 ReprojectionOffset, tsr_half2 ReprojectionBoundary)
{
	uint EncodedReprojectionOffset = EncodeReprojectionOffset(ReprojectionOffset);

	tsr_short2 BoundaryBits = tsr_short2(round(ReprojectionBoundary * tsr_half(1023))) + 1024;
	BoundaryBits = fastClamp(BoundaryBits, tsr_short(1).xx, tsr_short(2047).xx);

	uint EncodedReprojectionBoundary = EncodedReprojectionOffset | (uint(BoundaryBits.x) << 4) | (uint(BoundaryBits.y) << 15);
	return EncodedReprojectionBoundary;
}

/** Decodes EncodeReprojectionBoundary()'s ReprojectionOffset. */
tsr_short2 DecodeReprojectionOffset(uint EncodedReprojectionBoundary)
{
	tsr_short2 ReprojectionOffset = (tsr_short2(EncodedReprojectionBoundary, EncodedReprojectionBoundary >> 2u) & tsr_short(0x3)) - tsr_short(1);
	return ReprojectionOffset;
}

/** Returns any(EncodeReprojectionBoundary() != 0). */
bool HasEncodedReprojectionOffset(uint EncodedReprojectionBoundary)
{
	uint EncodedReprojectionOffset = EncodedReprojectionBoundary & 0xFu;
	bool bHasReprojectionOffset = EncodedReprojectionOffset != 5;
	return bHasReprojectionOffset;
}

/** Decodes EncodeReprojectionBoundary()'s ReprojectionBoundary. */
tsr_half2 DecodeReprojectionBoundary(uint EncodedReprojectionBoundary)
{
	tsr_short2 BoundaryBits = (tsr_short2(EncodedReprojectionBoundary >> 4u, EncodedReprojectionBoundary >> 15u) & tsr_short(0x7FF)) - tsr_short(1024);
	tsr_half2 ReprojectionBoundary = fastClamp(tsr_half2(BoundaryBits) * rcp(tsr_half(1023)), tsr_half(-1.0).xx, tsr_half(1.0).xx);
	return ReprojectionBoundary;
}

/** Estimate the coverage of the boundary over the rendered pixel. */
tsr_half ComputeReprojectionBoundaryCoverage(tsr_half2 ReprojectionBoundary)
{
	tsr_half ReprojectionBoundaryCoverage = length(ReprojectionBoundary);
	return ReprojectionBoundaryCoverage;
}

/** Returns whether the history's pixel has any overlap with the velocity boundary, to apply the ReprojectionOffset. */
bool IsHistoryPixelWithinOffsetBoundary(tsr_half2 dInputKO, tsr_half2 ReprojectionBoundary, float HistoryToInputFactor)
{
	tsr_half2 dInputKC = tsr_half(0.5) - tsr_half2(sign_bit(ReprojectionBoundary));
	tsr_half2 dInputOR = + dInputKC * tsr_half(HistoryToInputFactor);

	tsr_half2 dInputBC = ReprojectionBoundary;
	tsr_half2 dInputBR = dInputBC - dInputKC + dInputKO + dInputOR;

	bool bHistoryPixelWithinOffsetBoundary = dot(dInputBR, dInputBC) > tsr_half(0.0);
	return bHistoryPixelWithinOffsetBoundary;
}


//------------------------------------------------------- REPROJECTION's VECTOR

/** Encodes the reprojection vector into 32bits. */
uint EncodeReprojectionVector(float2 ReprojectionVector)
{
	float2 EncodedScreenVelocity = EncodeVelocityToTexture(float3(ReprojectionVector.xy, 0.0)).xy;
	
	float2 UnormEncoding = clamp(EncodedScreenVelocity * 65535.0f + 0.5f, 0.0f, 65535.0f);

	uint EncodedReprojectionVector = 0u;
	EncodedReprojectionVector |= uint(UnormEncoding.x) <<  0u;
	EncodedReprojectionVector |= uint(UnormEncoding.y) << 16u;
	return EncodedReprojectionVector;
}

/** Inverse of EncodeReprojectionVector() */
float2 DecodeReprojectionVector(uint EncodedReprojectionVector)
{
	float2 UnormEncoding = float2(uint2(EncodedReprojectionVector, EncodedReprojectionVector >> 16u) & 0xFFFFu);

	float2 ReprojectionVector = DecodeVelocityFromTexture(float4(UnormEncoding * rcp(65535.0f), 0.0, 0.0)).xy;
	return ReprojectionVector;
}


//------------------------------------------------------- REPROJECTION's JACOBIAN

static const tsr_half kVelocityJacobianRange = tsr_half(2.0);
//static const tsr_half kVelocityJacobianRange = tsr_half(0.5);
static const uint kVelocityJacobianBits = 8u;

/** Encodes the reprojection's jacobian into 32bits. */
uint EncodeReprojectionJacobian(tsr_half2x2 ReprojectionJacobian)
{
	const tsr_half EncodingRange = tsr_half((1u << (kVelocityJacobianBits - 1u)) - 1u);
	const tsr_half EncodingOffset = EncodingRange + tsr_half(0.5);
	const tsr_half EncodingMultiply = EncodingRange / kVelocityJacobianRange;

	ReprojectionJacobian[0] = clamp(ReprojectionJacobian[0], -kVelocityJacobianRange, kVelocityJacobianRange);
	ReprojectionJacobian[1] = clamp(ReprojectionJacobian[1], -kVelocityJacobianRange, kVelocityJacobianRange);

	#if 1
		ReprojectionJacobian[0] = tsr_half2(sign(ReprojectionJacobian[0])) * sqrt(abs(ReprojectionJacobian[0])) * sqrt(kVelocityJacobianRange);
		ReprojectionJacobian[1] = tsr_half2(sign(ReprojectionJacobian[1])) * sqrt(abs(ReprojectionJacobian[1])) * sqrt(kVelocityJacobianRange);
	#endif

	tsr_ushort2x2 DiscreteJacobian = tsr_ushort2x2(EncodingMultiply * ReprojectionJacobian + EncodingOffset);
	uint EncodedReprojectionJacobian = 0u;
	EncodedReprojectionJacobian |= uint(DiscreteJacobian[0][0]) << (0u * kVelocityJacobianBits);
	EncodedReprojectionJacobian |= uint(DiscreteJacobian[0][1]) << (1u * kVelocityJacobianBits);
	EncodedReprojectionJacobian |= uint(DiscreteJacobian[1][0]) << (2u * kVelocityJacobianBits);
	EncodedReprojectionJacobian |= uint(DiscreteJacobian[1][1]) << (3u * kVelocityJacobianBits);
	return EncodedReprojectionJacobian;
}

/** Inverse of EncodeReprojectionJacobian() */
tsr_half2x2 DecodeReprojectionJacobian(uint EncodedReprojectionJacobian)
{
	const uint ComponentMask = (1u << kVelocityJacobianBits) - 1u;
	const tsr_half EncodingRange = tsr_half((1u << (kVelocityJacobianBits - 1u)) - 1u);
	const tsr_half DecodingMultiply = kVelocityJacobianRange / EncodingRange;

	tsr_ushort2x2 DiscreteJacobian;
	DiscreteJacobian[0][0] = tsr_ushort((EncodedReprojectionJacobian >> (0u * kVelocityJacobianBits)) & ComponentMask);
	DiscreteJacobian[0][1] = tsr_ushort((EncodedReprojectionJacobian >> (1u * kVelocityJacobianBits)) & ComponentMask);
	DiscreteJacobian[1][0] = tsr_ushort((EncodedReprojectionJacobian >> (2u * kVelocityJacobianBits)) & ComponentMask);
	DiscreteJacobian[1][1] = tsr_ushort((EncodedReprojectionJacobian >> (3u * kVelocityJacobianBits)) & ComponentMask);
	
	tsr_half2x2 ReprojectionJacobian = tsr_half2x2(DiscreteJacobian) * DecodingMultiply - tsr_half(kVelocityJacobianRange);
	
	#if 1
		ReprojectionJacobian[0] = ReprojectionJacobian[0] * abs(ReprojectionJacobian[0]) * rcp(kVelocityJacobianRange);
		ReprojectionJacobian[1] = ReprojectionJacobian[1] * abs(ReprojectionJacobian[1]) * rcp(kVelocityJacobianRange);
	#endif

	return ReprojectionJacobian;
}

/** Estimate upscale ratio of the reprojection jsut from the jacobian. */
tsr_half ComputeReprojectionUpscaleFactorFromJacobian(tsr_half2x2 ReprojectionJacobian)
{
	tsr_half2 VectorOffsetE = tsr_half2(1.0, 0.0) + ReprojectionJacobian[0];
	tsr_half2 VectorOffsetS = tsr_half2(0.0, 1.0) + ReprojectionJacobian[1];

	tsr_half ReprojectionUpscaleFactor = sqrt(max(dot(VectorOffsetE, VectorOffsetE), tsr_half(1.0)) * max(dot(VectorOffsetS, VectorOffsetS), tsr_half(1.0)));
	return ReprojectionUpscaleFactor;
}
