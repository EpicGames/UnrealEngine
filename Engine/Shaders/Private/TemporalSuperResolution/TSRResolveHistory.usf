// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRColorSpace.ush"


//------------------------------------------------------- CONFIG

#if DIM_NYQUIST_WAVE_SIZE != 0
	#define WAVE_COUNT_X 1
	#define WAVE_COUNT_Y 1

	#define DOWNSAMPLE_FACTOR 2u
	
	/** Size of the entire group loaded into memory including overscan. */
	#define GROUP_TILE_SIZE 8u
	#define TILE_SIZE (GROUP_TILE_SIZE - 2u)

	#define LANE_OUTPUT_STRIDE_X 2u
	#if DIM_NYQUIST_WAVE_SIZE == 16
		#define LANE_OUTPUT_STRIDE_Y 2u
	#elif DIM_NYQUIST_WAVE_SIZE == 32
		#define LANE_OUTPUT_STRIDE_Y 1u
	#else
		#error Unknown wave size
	#endif
	#define LANE_INPUT_STRIDE_X (LANE_OUTPUT_STRIDE_X * DOWNSAMPLE_FACTOR)
	#define LANE_INPUT_STRIDE_Y (LANE_OUTPUT_STRIDE_Y * DOWNSAMPLE_FACTOR)
	
	/** Number of lane the GROUP_TILE_SIZE. */
	#define LANE_COUNT_X (GROUP_TILE_SIZE / LANE_OUTPUT_STRIDE_X)
	#define LANE_COUNT_Y (GROUP_TILE_SIZE / LANE_OUTPUT_STRIDE_Y)

	/** Size of the SIMD per lane that also number of input pixel loaded into each individual lanes. */
	#define INPUT_SIMD_SIZE (LANE_INPUT_STRIDE_X * LANE_INPUT_STRIDE_Y)
	#define OUTPUT_SIMD_SIZE (LANE_OUTPUT_STRIDE_X * LANE_OUTPUT_STRIDE_Y)

	#define tsr_input_halfC  TLaneVector2D<tsr_half, CONFIG_CHANNEL_COUNT, LANE_INPUT_STRIDE_X, LANE_INPUT_STRIDE_Y>
	#define tsr_output_halfC TLaneVector2D<tsr_half, CONFIG_CHANNEL_COUNT, LANE_OUTPUT_STRIDE_X, LANE_OUTPUT_STRIDE_Y>
	#define tsr_output_half  TLaneVector2D<tsr_half, 1, LANE_OUTPUT_STRIDE_X, LANE_OUTPUT_STRIDE_Y>

#else
	#define TILE_SIZE 8
#endif

#define KERNEL_SIZE 4

#define CONFIG_CLAMP 1

#define CONFIG_HDR_WEIGHT 1


//------------------------------------------------------- CONVOLUTIONS

#if DIM_NYQUIST_WAVE_SIZE != 0
	#include "TSRConvolutionNetwork.ush"
#endif


//------------------------------------------------------- CONSTANTS

static const int2 kOffsetsCross3x3[4] = {
	int2(-1, -1),
	int2( 1, -1),
	int2(-1,  1),
	int2( 1,  1),
};


//------------------------------------------------------- PARAMETERS

RWTexture2D<uint> PrevUseCountOutput;
RWTexture2D<uint> PrevClosestDepthOutput;


//------------------------------------------------------- LDS

groupshared tsr_halfC SharedArray0[TILE_SIZE * TILE_SIZE];


//------------------------------------------------------- FUNCTIONS

static const float MitchellNetravaliB = rcp(3.0);
static const float MitchellNetravaliC = rcp(3.0);

// Mitchell Netravali
// B-spline:       B=1      C=0
// Mitchell:       B=1/3    C=1/3
// Catmull-Rom:    B=0      C=1/2
// Robidoux:       B=0.3782 C=0.3109 (cylindrical)
// Robidoux Sharp: B=0.2620 C=0.3690 (cylindrical)
// Robidoux Soft:  B=0.6796 C=0.1602 (cylindrical)
void MitchellNetravaliCoefs(const float B, const float C, out float OutQ0[4], out float OutQ1[4])
{
	OutQ0[0] = (6.0 - 2.0 * B) / 6.0;
	OutQ0[1] = 0.0;
	OutQ0[2] = (-18.0 + 12.0 * B + 6.0 * C) / 6.0;
	OutQ0[3] = (12.0 - 9.0 * B - 6.0 * C) / 6.0;

	OutQ1[0] = (8 * B + 24 * C) / 6.0;
	OutQ1[1] = (-12 * B - 48 * C) / 6.0;
	OutQ1[2] = (6 * B + 30 * C) / 6.0;
	OutQ1[3] = (-B - 6 * C) / 6.0;
}

float MitchellNetravali(float d, const float B, const float C)
{
	// Coeficient ends up known at compile time.
	float Q0[4];
	float Q1[4];
	MitchellNetravaliCoefs(B, C, Q0, Q1);

	if (d < 1)
	{
		return Q0[0] + d * (Q0[1] + d * (Q0[2] + d * Q0[3]));
	}
	else if ((d >= 1) && (d < 2))
	{
		return Q1[0] + d * (Q1[1] + d * (Q1[2] + d * Q1[3]));
	}
	else
	{
		return 0;
	}
}


//------------------------------------------------------- PARAMETERS

FScreenTransform DispatchThreadToHistoryPixelPos;
uint2 OutputViewRectMin;
uint2 OutputViewRectMax;
uint bGenerateOutputMip1;
float HistoryValidityMultiply;

Texture2D<tsr_halfC> UpdateHistoryOutputTexture;

RWTexture2D<tsr_halfC> SceneColorOutputMip0;
RWTexture2D<tsr_halfC> SceneColorOutputMip1;


//------------------------------------------------------- ENTRY POINT NYQUIST DOWNSAMPLE

#if DIM_NYQUIST_WAVE_SIZE != 0

#if COMPILER_SUPPORTS_WAVE_SIZE
	WAVESIZE(DIM_NYQUIST_WAVE_SIZE)
#endif
[numthreads(LANE_COUNT, 1, 1)]
void ResolveHistoryCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	const uint LaneIndex = GroupThreadIndex;

	// Load all the colors
	tsr_input_halfC Colors;
	ISOLATE
	{
		const tsr_short2 GroupPixelOffset = (
			tsr_short2(HistoryInfo_ViewportMin) +
			tsr_short2(GroupId) * tsr_short(TILE_SIZE * DOWNSAMPLE_FACTOR).xx +
			tsr_short(-1).xx);
		
		UNROLL_N(INPUT_SIMD_SIZE)
		for (uint SimdIndex = 0; SimdIndex < INPUT_SIMD_SIZE; SimdIndex++)
		{
			tsr_short2 PixelPos = GroupPixelOffset + GetLaneSimdPixelOffset<LANE_INPUT_STRIDE_X, LANE_INPUT_STRIDE_Y>(LaneIndex, SimdIndex);
			PixelPos = clamp(PixelPos, tsr_short2(HistoryInfo_ViewportMin), tsr_short2(HistoryInfo_ViewportMax - 1));

			Colors.SetElement(SimdIndex, UpdateHistoryOutputTexture[PixelPos]);
		}
	}

	tsr_output_halfC Output;
	{
		#if CONFIG_CLAMP
			tsr_output_halfC Min = DownsampleMin2x2(Colors);
			tsr_output_halfC Max = DownsampleMax2x2(Colors);

			Min = min(Min, WaveAccessNeighborTexel(Min, tsr_short2(1, 0)));
			Min = min(Min, WaveAccessNeighborTexel(Min, tsr_short2(0, 1)));

			Max = max(Max, WaveAccessNeighborTexel(Max, tsr_short2(1, 0)));
			Max = max(Max, WaveAccessNeighborTexel(Max, tsr_short2(0, 1)));
		#endif
		
		tsr_input_halfC LDRColors = Colors;
		#if CONFIG_HDR_WEIGHT
		{
			UNROLL_N(INPUT_SIMD_SIZE)
			for (uint SimdIndex = 0; SimdIndex < INPUT_SIMD_SIZE; SimdIndex++)
			{
				LDRColors.SetElement(SimdIndex, Colors.GetElement(SimdIndex) * HdrWeight4(Colors.GetElement(SimdIndex)));
			}
		}
		#endif

		/*
		 *  ^
		 *  |  b  c
		 *  |  a  b
		 *  o------>
		 */
		const float WeightA = MitchellNetravali(/* PixelOffset = */ 0.5, MitchellNetravaliB, MitchellNetravaliC);
		const float WeightB = MitchellNetravali(/* PixelOffset = */ 1.5, MitchellNetravaliB, MitchellNetravaliC);
		const float TotalWeight = 4.0 * (WeightA * WeightA + WeightB * WeightB + WeightA * WeightB * 2.0);

		const tsr_half FinalWeightA = tsr_half(WeightA * WeightA / TotalWeight);
		const tsr_half FinalWeightB = tsr_half(WeightA * WeightB / TotalWeight);
		const tsr_half FinalWeightC = tsr_half(WeightB * WeightB / TotalWeight);
	
		/*
		 *  c   b | b   c
		 *  b   a | a   b
		 *  ------o------
		 *  b   a | a   b
		 *  c   b | b   c
		 */
		const tsr_half Weights0[4] = { FinalWeightC, FinalWeightB, FinalWeightB, FinalWeightA };
		const tsr_half Weights1[4] = { FinalWeightB, FinalWeightC, FinalWeightA, FinalWeightB };
		const tsr_half Weights2[4] = { FinalWeightB, FinalWeightA, FinalWeightC, FinalWeightB };
		const tsr_half Weights3[4] = { FinalWeightA, FinalWeightB, FinalWeightB, FinalWeightC };

		tsr_output_halfC Out0 = DownsampleDot2x2(LDRColors, Weights0);
		tsr_output_halfC Out1 = DownsampleDot2x2(LDRColors, Weights1);
		tsr_output_halfC Out2 = DownsampleDot2x2(LDRColors, Weights2);
		tsr_output_halfC Out3 = DownsampleDot2x2(LDRColors, Weights3);

		Output = Out0 + WaveAccessNeighborTexel(Out1, tsr_short2(1, 0)) + WaveAccessNeighborTexel(Out2, tsr_short2(0, 1)) + WaveAccessNeighborTexel(Out3, tsr_short2(1, 1));
		#if CONFIG_HDR_WEIGHT
		{
			UNROLL_N(OUTPUT_SIMD_SIZE)
			for (uint SimdIndex = 0; SimdIndex < OUTPUT_SIMD_SIZE; SimdIndex++)
			{
				Output.SetElement(SimdIndex, Output.GetElement(SimdIndex) * HdrWeightInvY(Luma4(Output.GetElement(SimdIndex))));
			}
		}
		#endif
		#if CONFIG_CLAMP
			Output = clamp(Output, Min, Max);
		#endif
	}

	ISOLATE
	{
		const tsr_short2 GroupPixelOffset = (
			tsr_short2(OutputViewRectMin) +
			tsr_short2(GroupId) * tsr_short(TILE_SIZE).xx);
		
		Output = clamp(Output, tsr_output_halfC::Const(tsr_half(0.0)), tsr_output_halfC::Const(tsr_half(Max10BitsFloat)));

		// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
		// (0.995 chosen to accommodate handling of 254/255)
		#if CONFIG_SCENE_COLOR_ALPHA
		{
			Output.SetComponent(3, select(Output[3] > tsr_output_half::Const(0.995), tsr_output_half::Const(1.0), Output[3]));
			Output.SetComponent(3, select(Output[3] < tsr_output_half::Const(0.005), tsr_output_half::Const(0.0), Output[3]));
		}
		#endif

		UNROLL_N(OUTPUT_SIMD_SIZE)
		for (uint SimdIndex = 0; SimdIndex < OUTPUT_SIMD_SIZE; SimdIndex++)
		{
			tsr_short2 OutputPixelOffset = GetLaneSimdPixelOffset<LANE_OUTPUT_STRIDE_X, LANE_OUTPUT_STRIDE_Y>(LaneIndex, SimdIndex);
			tsr_short2 OutputPixelPos = InvalidateOutputPixelPos(GroupPixelOffset + OutputPixelOffset, OutputViewRectMax);
			
			OutputPixelPos.x = select(all(OutputPixelOffset < TILE_SIZE), OutputPixelPos.x, ~tsr_short(0));

			SceneColorOutputMip0[OutputPixelPos] = Output.GetElement(SimdIndex);
		}
		
		{
			tsr_output_halfC HalfResOutput = Output * tsr_half(0.25);
			// Forces the * tsr_half(0.25) to be applied before to avoid turning bright pixels to +inf in the adds below.
			#if CONFIG_FP16_PRECISE_MULTIPLY_ORDER
				HalfResOutput = min(HalfResOutput, tsr_output_halfC::Const(tsr_half(Max10BitsFloat * 0.25)));
			#endif

			HalfResOutput = HalfResOutput + WaveAccessNeighborTexel(HalfResOutput, tsr_short2(1, 0));
			HalfResOutput = HalfResOutput + WaveAccessNeighborTexel(HalfResOutput, tsr_short2(0, 1));
		
			tsr_short2 OutputPixelOffset = GetLaneSimdPixelOffset<LANE_OUTPUT_STRIDE_X, LANE_OUTPUT_STRIDE_Y>(LaneIndex, /* SimdIndex = */ 0);
			tsr_short2 OutputPixelPos = InvalidateOutputPixelPos(GroupPixelOffset + OutputPixelOffset, OutputViewRectMax);
			OutputPixelPos.x = select(all(OutputPixelOffset < TILE_SIZE), OutputPixelPos.x, ~tsr_short(0));

			tsr_short2 HalfResOutputPixelPos = OutputPixelPos >> tsr_short(1);
			HalfResOutputPixelPos[0] |= ((OutputPixelPos[0] & tsr_short(0x1)) | tsr_short(!bGenerateOutputMip1)) * tsr_short(~0);
			HalfResOutputPixelPos[1] |= ((OutputPixelPos[1] & tsr_short(0x1)) | tsr_short(!bGenerateOutputMip1)) * tsr_short(~0);
			
			SceneColorOutputMip1[HalfResOutputPixelPos] = HalfResOutput.GetElement(/* SimdIndex = */ 0);
		}
	}
}

#else // DIM_NYQUIST_WAVE_SIZE == 0

//------------------------------------------------------- ENTRY POINT REFERENCE

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void ResolveHistoryCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float4 Debug = 0.0;

	uint2 DispatchThreadId = (
		ZOrder2D(GroupThreadIndex, uint(log2(float(TILE_SIZE)))) +
		GroupId * uint2(TILE_SIZE, TILE_SIZE));

	tsr_halfC Color = tsr_half(0.0);
	
	// Downsample the history
	{
		// Position of the output pixel in the history.
		float2 OutputPixelPos = ApplyScreenTransform(float2(DispatchThreadId), DispatchThreadToHistoryPixelPos);
		float2 TopLeftKernel = floor(OutputPixelPos + (0.5 - 0.5 * KERNEL_SIZE)) + 0.5;
		
		#if CONFIG_CLAMP
			tsr_halfC MinColor = tsr_half(POSITIVE_INFINITY);
			tsr_halfC MaxColor = tsr_half(0.0);
		#endif
		tsr_half ColorWeight = tsr_half(0.0);

		UNROLL_N(KERNEL_SIZE)
		for (uint x = 0; x < KERNEL_SIZE; x++)
		{
			UNROLL_N(KERNEL_SIZE)
			for (uint y = 0; y < KERNEL_SIZE; y++)
			{
				float2 SamplePixelPos = TopLeftKernel + float2(x, y);

				float2 SampleUV = SamplePixelPos * HistoryInfo_ExtentInverse;

				SampleUV = clamp(SampleUV, HistoryInfo_UVViewportBilinearMin, HistoryInfo_UVViewportBilinearMax);

				tsr_halfC SampleColor = UpdateHistoryOutputTexture.SampleLevel(GlobalBilinearClampedSampler, SampleUV, 0);
				
				float2 PixelOffset = abs(SamplePixelPos - OutputPixelPos);

				tsr_half kernelWeight = tsr_half(MitchellNetravali(PixelOffset.x, MitchellNetravaliB, MitchellNetravaliC) * MitchellNetravali(PixelOffset.y, MitchellNetravaliB, MitchellNetravaliC));
				tsr_half SampleHdrWeight = HdrWeight4(SampleColor);
				
				#if CONFIG_CLAMP
					MinColor = min(MinColor, SampleColor);
					MaxColor = max(MaxColor, SampleColor);
				#endif
				Color  += (kernelWeight * SampleHdrWeight) * SampleColor;
				ColorWeight += (kernelWeight * SampleHdrWeight);
			}
		}

		Color *= SafeRcp(ColorWeight);
		#if CONFIG_CLAMP
			Color = clamp(Color, MinColor, MaxColor);
		#endif
	}
	
	// Compute final output
	tsr_halfC FinalOutputColor = Color;
	
	FinalOutputColor = clamp(FinalOutputColor, tsr_half(0.0), tsr_half(Max10BitsFloat));

	// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
	// (0.995 chosen to accommodate handling of 254/255)
	#if CONFIG_SCENE_COLOR_ALPHA
	{
		FinalOutputColor[3] = select(FinalOutputColor[3] > tsr_half(0.995), tsr_half(1.0), FinalOutputColor[3]);
		FinalOutputColor[3] = select(FinalOutputColor[3] < tsr_half(0.005), tsr_half(0.0), FinalOutputColor[3]);
	}
	#endif

	tsr_short2 OutputPixelPosition = InvalidateOutputPixelPos(tsr_short2(OutputViewRectMin + DispatchThreadId), OutputViewRectMax);
	
	// Output final scene color Mip0
	{
		SceneColorOutputMip0[OutputPixelPosition] = FinalOutputColor;
	}

	// Output final scene color Mip1
	{
		uint LocalGroupThreadIndex = GroupThreadIndex;

		tsr_halfC HalfResOutput = FinalOutputColor * tsr_half(0.25);
		// Forces the * tsr_half(0.25) to be applied before to avoid turning bright pixels to +inf in the adds below.
		#if CONFIG_FP16_PRECISE_MULTIPLY_ORDER
			HalfResOutput = min(HalfResOutput, tsr_half(Max10BitsFloat * 0.25));
		#endif

		tsr_short2 HalfResOutputPixelPos;
		HalfResOutputPixelPos[0] = (OutputPixelPosition[0] >> tsr_short(1)) | (((OutputPixelPosition[0] & tsr_short(0x1)) | tsr_short(!bGenerateOutputMip1)) * tsr_short(~0));
		HalfResOutputPixelPos[1] = (OutputPixelPosition[1] >> tsr_short(1)) | (((OutputPixelPosition[1] & tsr_short(0x1)) | tsr_short(!bGenerateOutputMip1)) * tsr_short(~0));
			
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST && 0 // Support WaveBroadcast with halfs
		BRANCH
		if (bGenerateOutputMip1)
		{
			FWaveBroadcastSettings Horizontal = InitWaveXorButterfly(/* XorButterFly = */ 0x1);
			FWaveBroadcastSettings Vertical = InitWaveXorButterfly(/* XorButterFly = */ 0x2);

			HalfResOutput += WaveBroadcast(Horizontal, HalfResOutput);
			HalfResOutput += WaveBroadcast(Vertical, HalfResOutput);
		}
		#else
		BRANCH
		if (bGenerateOutputMip1)
		{
			SharedArray0[LocalGroupThreadIndex] = HalfResOutput;

			#if COMPILER_METAL
				GroupMemoryBarrierWithGroupSync();
			#endif

			SharedArray0[LocalGroupThreadIndex] += SharedArray0[LocalGroupThreadIndex ^ 0x1];
			SharedArray0[LocalGroupThreadIndex] += SharedArray0[LocalGroupThreadIndex ^ 0x2];
			
			HalfResOutput = SharedArray0[LocalGroupThreadIndex];
		}
		#endif

		SceneColorOutputMip1[HalfResOutputPixelPos] = HalfResOutput;
	}

	#if DEBUG_OUTPUT
	{
		DebugOutput[tsr_short3(OutputPixelPosition, 0)] = Debug;
	}
	#endif
}

#endif // DIM_NYQUIST_WAVE_SIZE == 0
