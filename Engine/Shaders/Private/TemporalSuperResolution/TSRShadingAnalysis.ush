// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//------------------------------------------------------- INCLUDES

#include "TSRColorSpace.ush"
#include "TSRSpatialAntiAliasing.ush"
#include "TSRConvolutionNetworkPass.ush"

//------------------------------------------------------- CONFIG

#ifndef CONFIG_THIN_GEOMETRY_DETECTION
#define CONFIG_THIN_GEOMETRY_DETECTION 0
#endif

#define REJECT_SHADING_RESPONSIVE	0
#define REJECT_SHADING_STABILITY	1

//------------------------------------------------------- CONSTANTS

static const float kHistoryGuidePreceptionAdd = 1.0;

static const tsr_half FilteringWeight = rcp(tsr_half(1.0 + 4 * 0.5 + 4 * 0.25));


//------------------------------------------------------- PARAMETERS
float ExposureOffsetFactor;
float FlickeringFramePeriod;
uint bPassthroughAlpha;
uint RejectShadingMode;
int ThinGeometryAffectMoireError;

// Cache the threshold at which the variation count should quick in
float2 TotalVariationCountInvThreshold;

//------------------------------------------------------- COMPOSE SEPARATE TRANSLUCENCY

/** Compose translucency in linear color space. */
CALL_SITE_DEBUGLOC
tsr_tensor_halfC ComposeTranslucency(tsr_tensor_halfC Color, tsr_tensor_half4 Translucency, tsr_tensor_bool bAllowPassthroughAlpha = tsr_tensor_bool::Const(true))
{
	tsr_tensor_halfC ComposedColor;
	ComposedColor = Color * tsr_tensor_halfC::Vectorize(Translucency[3]) + ResizeChannels<CONFIG_CHANNEL_COUNT>(Translucency);
	#if CONFIG_SCENE_COLOR_ALPHA
		ComposedColor.SetComponent(3, select(and(tsr_tensor_bool::Const(bPassthroughAlpha > 0), bAllowPassthroughAlpha), Color[3], Color[3] * Translucency[3]));
	#endif
	#if CONFIG_SCENE_COLOR_ALPHA
		ComposedColor = min(ComposedColor, tsr_tensor_halfC::Const(LargestSceneColorRGBA));
	#else
		ComposedColor = min(ComposedColor, tsr_tensor_halfC::Const(LargestSceneColorRGB));
	#endif
	return ComposedColor;
}

/** Compose blurry translucency for tighter rejection in linear color space. */
tsr_tensor_halfC ComposeTranslucencyForRejection(tsr_tensor_halfC OriginalOpaqueInput, tsr_tensor_half4 OriginalTranslucencyInput, tsr_tensor_bool bHasPixelAnimation)
{
	tsr_tensor_halfC SharpInput = OriginalOpaqueInput;
	tsr_tensor_half4 BlurInput = OriginalTranslucencyInput;

	{
		#if CONFIG_SCENE_COLOR_ALPHA
			const tsr_half4 TransparentSharpInput = tsr_half4(0.0, 0.0, 0.0, 1.0);
		#else
			const tsr_half3 TransparentSharpInput = tsr_half(0.0).xxx;
		#endif

		tsr_tensor_halfC CenterFinalSceneColor = ComposeTranslucency(OriginalOpaqueInput, OriginalTranslucencyInput);
			
		tsr_tensor_half4 HasAnimationBlurInput = ResizeChannels<4>(CenterFinalSceneColor);
			
		SharpInput = select(tsr_tensor_boolC::Vectorize(bHasPixelAnimation), tsr_tensor_halfC::Const(TransparentSharpInput), SharpInput);
		BlurInput  = select(tsr_tensor_bool4::Vectorize(bHasPixelAnimation), HasAnimationBlurInput, BlurInput);
	}

#if CONFIG_SCENE_COLOR_ALPHA
	// Disallow alpha passthrough on opaque materials with pixel animation to retain scene color alpha values (i.e. dynamic holdout state).
	const tsr_tensor_bool bAllowPassthroughAlpha = not(bHasPixelAnimation);

	return ComposeTranslucency(SharpInput, Blur3x3(BlurInput), bAllowPassthroughAlpha);
#else
	return ComposeTranslucency(SharpInput, Blur3x3(BlurInput));
#endif
}


//------------------------------------------------------- CHANGE LDR EXPOSURE

CALL_SITE_DEBUGLOC
tsr_tensor_halfC RemovePerceptionAdd(tsr_tensor_halfC SrcColor, const float SourcePerceptionAdd)
{
	tsr_tensor_halfC Minus = tsr_tensor_halfC::Const(tsr_half(1.0)) - SrcColor;
	tsr_tensor_halfC LDRToLinear = min(tsr_tensor_halfC::Const(tsr_half(SourcePerceptionAdd)) * rcp(Minus), tsr_tensor_halfC::Const(tsr_half(MaxHalfFloat)));

	return SrcColor * LDRToLinear;
}

CALL_SITE_DEBUGLOC
tsr_tensor_halfC AddPerceptionAdd(tsr_tensor_halfC SrcColor, const float DestPerceptionAdd)
{
	SrcColor = SrcColor * rcp(SrcColor + tsr_tensor_halfC::Const(tsr_half(DestPerceptionAdd)));
	return SrcColor;
}


//------------------------------------------------------- GUIDE COLOR SPACE

CALL_SITE_DEBUGLOC
tsr_tensor_halfC LinearToGCS(tsr_tensor_halfC SrcColor)
{
	tsr_tensor_halfC GColor = SrcColor * rcp(SrcColor + tsr_tensor_halfC::Const(tsr_half(0.17)));
	//SMColor = SMColor * SMColor;
	#if CONFIG_SCENE_COLOR_ALPHA
		GColor.SetComponent(3, SrcColor[3]);
	#endif
	return GColor;
}

CALL_SITE_DEBUGLOC
tsr_tensor_halfC GCSToLinear(tsr_tensor_halfC GColor)
{
	//SrcColor = sqrt(SrcColor);

	tsr_tensor_halfC Minus = tsr_tensor_halfC::Const(tsr_half(1.0)) - GColor;
	tsr_tensor_halfC LDRToLinear = min(tsr_tensor_halfC::Const(tsr_half(0.17)) * rcp(Minus), tsr_tensor_halfC::Const(tsr_half(MaxHalfFloat)));

	tsr_tensor_halfC LinearColor = GColor * LDRToLinear;
	#if CONFIG_SCENE_COLOR_ALPHA
		LinearColor.SetComponent(3, GColor[3]);
	#endif
	return LinearColor;
}

//------------------------------------------------------- SHADING MEASUREMENT COLOR SPACE

CALL_SITE_DEBUGLOC
tsr_tensor_half GCSToSMCS(tsr_tensor_half GColor)
{
	tsr_tensor_half SMColor = GColor * GColor;
	return SMColor;
}

CALL_SITE_DEBUGLOC
tsr_tensor_half SMCSToGCS(tsr_tensor_half SMColor)
{
	tsr_tensor_half GColor = sqrt(SMColor);
	return GColor;
}

CALL_SITE_DEBUGLOC
tsr_tensor_halfC GCSToSMCS(tsr_tensor_halfC GColor)
{
	tsr_tensor_halfC SMColor = GColor * GColor;
	#if CONFIG_SCENE_COLOR_ALPHA
		SMColor.SetComponent(3, GColor[3]);
	#endif
	return SMColor;
}

CALL_SITE_DEBUGLOC
tsr_tensor_halfC LinearToSMCS(tsr_tensor_halfC LinearColor)
{
	return GCSToSMCS(LinearToGCS(LinearColor));
}


//------------------------------------------------------- SPATIAL ANTI-ALIASER

// Compute the LDR luminance the spatial anti-aliaser should anti-alias.
CALL_SITE_DEBUGLOC
tsr_tensor_half ComputeSpatialAntiAliaserLumaLDR(tsr_tensor_halfC SceneColor)
{
	const tsr_half SpatialAAExposure = tsr_half(0.5);
		
	tsr_tensor_half AALumaLDR;

	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_half PixelLuma = dot(SceneColor.GetElement(ElementIndex).rgb, tsr_half3(0.299f, 0.587f, 0.114f));
		AALumaLDR.SetElement(ElementIndex, PixelLuma / (SpatialAAExposure + PixelLuma));
	}

	return AALumaLDR;
}

// Returns whether the aliasing is visible at all, to avoid paying spatial-antialiaser on low contrasts areas.
CALL_SITE_DEBUGLOC
tsr_tensor_bool IsAliasingVisible(tsr_tensor_halfC ExposedInputBoxSize)
{
	#if CONFIG_SCENE_COLOR_ALPHA
		tsr_tensor_half ExposedInputLuminanceBox = dot(ExposedInputBoxSize, tsr_tensor_halfC::Const(tsr_halfC(0.299f, 0.587f, 0.114f, 0.0)));
	#else
		tsr_tensor_half ExposedInputLuminanceBox = dot(ExposedInputBoxSize, tsr_tensor_halfC::Const(tsr_halfC(0.299f, 0.587f, 0.114f)));
	#endif

	tsr_tensor_bool bAliasingIsVisible = ExposedInputLuminanceBox > tsr_tensor_half::Const(SPATIAL_ANTI_ALIASER_MIN_LUMIMANCE);

	return bAliasingIsVisible;
}

// Whether to run spatial anti-aliaser on the frame. Must match SpatialAntiAliasingLerp.
CALL_SITE_DEBUGLOC
tsr_tensor_bool ShouldSpatialAntiAlias(tsr_tensor_bool bIsDisoccluded, tsr_tensor_bool bResurrectHistory, tsr_tensor_bool bAliasingIsVisible, tsr_tensor_half RejectionBlendFinal)
{
	tsr_tensor_bool bSpatialAntiAlias = and(bAliasingIsVisible, or(RejectionBlendFinal < tsr_tensor_half::Const(tsr_half(0.25)), and(bIsDisoccluded, not(bResurrectHistory))));
	
	return bSpatialAntiAlias;
}

//------------------------------------------------------- FLICKERING TEMPORAL ANALYSIS

void ComputeMoireError(
	tsr_tensor_bool  bIsDisoccluded,
	tsr_tensor_half  bIsStatic,
	tsr_tensor_half  MoireInput,
	tsr_tensor_halfC OriginalOpaqueInput,
	tsr_tensor_half4 OriginalTranslucencyInput,
	tsr_tensor_half4 PrevMoireHistory,
#if CONFIG_THIN_GEOMETRY_DETECTION
	tsr_tensor_half HistoryRelaxationFactor,
	const bool bEnableThinGeometry,
#endif
	out tsr_tensor_half4 OutMoireHistory,
	out tsr_tensor_half  OutMoireError)
{
	const tsr_half MinBlendFinal = tsr_half(0.05);
	const tsr_half MoireEncodingError = tsr_half(rcp(127.0));
	const tsr_half MaxPrevTotalVariationCount = tsr_half(20.0);
     const tsr_tensor_half ValidGeometryGradientThreshold = tsr_tensor_half::Const(tsr_half(0.00007));

	// Input luminance to monitor flickering from.
	tsr_tensor_half InputLuma = GCSToSMCS(MoireInput);

	// How much input luminance from the TSRComputeMoireLuma.usf measurement has been modified since for instance with fog, water.
	tsr_tensor_half InputLumaModification;
	{
		const tsr_half InputLumaEncodingError = tsr_half(0.5 / 255.0);
			
		tsr_tensor_halfC OriginalInput = ComposeTranslucency(OriginalOpaqueInput, OriginalTranslucencyInput);

		OriginalInput       = LinearToSMCS(OriginalInput * tsr_half(ExposureOffsetFactor));
		OriginalOpaqueInput = LinearToSMCS(OriginalOpaqueInput * tsr_half(ExposureOffsetFactor));

		tsr_tensor_half SceneColorLuma;
		tsr_tensor_half TranslucencyChange;
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			SceneColorLuma.SetElement(ElementIndex, dot(OriginalOpaqueInput.GetElement(ElementIndex).rgb, kMoireSMCSWeights));
			TranslucencyChange.SetElement(ElementIndex, dot(abs(OriginalOpaqueInput.GetElement(ElementIndex).rgb - OriginalInput.GetElement(ElementIndex).rgb), kMoireSMCSWeights));
		}

		InputLumaModification = tsr_tensor_half::Const(0.0);
		InputLumaModification = max(InputLumaModification, abs(InputLuma - SceneColorLuma) - tsr_tensor_half::Const(InputLumaEncodingError));
		InputLumaModification = max(InputLumaModification, TranslucencyChange);
		InputLumaModification = max(InputLumaModification, tsr_tensor_half::Const(1.0) - OriginalTranslucencyInput[3]);
	}

	// Unpack history
	tsr_tensor_half PrevFlickeringHistory = GCSToSMCS(PrevMoireHistory[0]);
	tsr_tensor_half PrevGradient = PrevMoireHistory[1] * tsr_half(255.0 / 127.0) - tsr_half(1.0);
	//tsr_tensor_half PrevTotalVariation = PrevMoireHistory[2] * tsr_tensor_half::Const(1.0 - MinBlendFinal);
	tsr_tensor_half PrevTotalVariation = PrevMoireHistory[2] * tsr_tensor_half::Const(tsr_half(1.0) - MinBlendFinal);
	tsr_tensor_half PrevTotalVariationCount = PrevMoireHistory[3] * MaxPrevTotalVariationCount * tsr_tensor_half::Const(tsr_half(1.0) - MinBlendFinal);
	
	// Discard history on moving objects.
	PrevTotalVariation = PrevTotalVariation * bIsStatic;
	PrevTotalVariationCount = PrevTotalVariationCount * bIsStatic;

	// Run the exact same heuristic as MeasureRejection on a single channel.
	tsr_tensor_half FilteredInputLumaModification;
	tsr_tensor_half OutRejectionBlendFinal;
	tsr_tensor_half OutRejectionClampBlend;
	{
		tsr_tensor_half BackbufferQuantizationErrorVector = tsr_tensor_half::Const(MeasureBackbufferLDRQuantizationError());
		
		tsr_tensor_half InputC0 = InputLuma;
		tsr_tensor_half HistoryC0 = PrevFlickeringHistory;

		// Anhilate history
		tsr_tensor_half InputC2 = InputC0;
		tsr_tensor_half HistoryC2 = HistoryC0;
		AnnihilateMutuallySingleChannel3x3(InputLuma, HistoryC0, /* out */ InputC2, /* out */ HistoryC2);

		// Convolve the input luma to have the exact same moire pattern as FilteredInput below.
		tsr_tensor_half FilteredInput;
		tsr_tensor_half FilteredHistory;
		Deconcatenate(Blur3x3(Concatenate(InputC2, InputLumaModification, HistoryC2)), /* out */ FilteredInput, /* out */ FilteredInputLumaModification, /* out */ FilteredHistory);
		
		// Compute clamping box of the limance.
		tsr_tensor_half FilteredBoxMin;
		tsr_tensor_half FilteredBoxMax;
		tsr_tensor_half InputC2BoxSize;
		{
			tsr_tensor_half2 Min;
			tsr_tensor_half2 Max;
			MinMax3x3(Concatenate(FilteredInput, InputC2), /* out */ Min, /* out */ Max);
			FilteredBoxMin = Min[0];
			FilteredBoxMax = Max[0];
		
		#if CONFIG_THIN_GEOMETRY_DETECTION
			// The scene can still flicker with thin geometry detection. Add thin geometry logic so that the moire error can
			// detect the residual flickering.
			if (bEnableThinGeometry)
			{
				// VGPR optimization for LerpClamp3x3()
				HistoryRelaxationFactor = select(HistoryRelaxationFactor > tsr_tensor_half::Const(tsr_half(1.0f / 127.0f)), HistoryRelaxationFactor, tsr_tensor_half::Const(0));
				tsr_tensor_half LerpFactorVector = HistoryRelaxationFactor;
		
				PLATFORM_SPECIFIC_ISOLATE
				{
					tsr_tensor_half ToClampBoundaryMin;
					tsr_tensor_half ToClampBoundaryMax;
					MinMax3x3(FilteredHistory, /*out*/ToClampBoundaryMin, /*out*/ToClampBoundaryMax);

					FilteredBoxMin = lerp(FilteredBoxMin, ToClampBoundaryMin, LerpFactorVector);
					FilteredBoxMax = lerp(FilteredBoxMax, ToClampBoundaryMax, LerpFactorVector);
				}
		
				tsr_tensor_half LerpClampedHistoryInput = fastClamp(FilteredHistory, FilteredBoxMin, FilteredBoxMax);

				{
					tsr_tensor_half ToClampBoundaryMin;
					tsr_tensor_half ToClampBoundaryMax;
					MinMax3x3(LerpClampedHistoryInput, /*out*/ToClampBoundaryMin, /*out*/ToClampBoundaryMax);

					FilteredBoxMin = select(LerpFactorVector > tsr_tensor_half::Const(0), ToClampBoundaryMin, FilteredBoxMin);
					FilteredBoxMax = select(LerpFactorVector > tsr_tensor_half::Const(0), ToClampBoundaryMax, FilteredBoxMax);
				}
			}
		#endif

			InputC2BoxSize = Max[1] - Min[1];
		}

		// Compute the clamp error
		tsr_tensor_half ClampError;
		{
		
			ClampError = BackbufferQuantizationErrorVector;
			
			BRANCH
			if (RejectShadingMode == REJECT_SHADING_RESPONSIVE)
			{
				tsr_tensor_half TotalVarInputDiffC0C2;
				tsr_tensor_half TotalVarInputC2;
				Deconcatenate(abs(TotalVariation3x3(Concatenate(abs(InputC0 - InputC2), InputC2))), /* out */ TotalVarInputDiffC0C2, /* out */ TotalVarInputC2);
				ClampError = max(ClampError, Blur3x3(min(TotalVarInputDiffC0C2, TotalVarInputC2)));
			}	

			ClampError = max(ClampError, InputC2BoxSize * tsr_half(FilteringWeight * 0.25));
			ClampError = ClampError + BackbufferQuantizationErrorVector;
		}

		// Apply clamp error
		FilteredBoxMin = FilteredBoxMin - ClampError;
		FilteredBoxMax = FilteredBoxMax + ClampError;

		tsr_tensor_half BoxSize = (
			InputC2BoxSize * tsr_tensor_half::Const(FilteringWeight) +
			(BackbufferQuantizationErrorVector * tsr_half(2 * FilteringWeight)));

		tsr_tensor_half ClampedFilteredHistory = fastClamp(FilteredHistory, FilteredBoxMin, FilteredBoxMax);

		tsr_tensor_half Delta = max(abs(FilteredInput - FilteredHistory), BoxSize);
	
		tsr_tensor_half RawClampedEnergy = abs(ClampedFilteredHistory - FilteredHistory);
		tsr_tensor_half RawRejection     = saturate(tsr_tensor_half::Const(1.0) - max(RawClampedEnergy, tsr_tensor_half::Const(0.0)) * rcp(Delta));

		tsr_tensor_half FilteredClampedEnergy;
		Deconcatenate(MaxRMinG3x3(Median3x3(Concatenate(RawClampedEnergy, RawRejection))), /* out */ FilteredClampedEnergy, /* out */ OutRejectionClampBlend);

		OutRejectionBlendFinal = saturate(tsr_tensor_half::Const(1.0) - max(FilteredClampedEnergy, tsr_tensor_half::Const(0.0)) * rcp(Delta));
	}
	
	// Simulate what happens in TSRUpdateHistory.usf
	tsr_tensor_half FinalFlickeringHistory;
	tsr_tensor_half CurrentGradient;
	{
		tsr_tensor_half ClampedPrevFlickeringHistory = ClampPlus3x3(PrevFlickeringHistory, InputLuma);
		tsr_tensor_half FinalPrevFlickeringHistory = lerp(ClampedPrevFlickeringHistory, PrevFlickeringHistory, OutRejectionClampBlend);

		tsr_tensor_half FinalBlendFinal = max(tsr_tensor_half::Const(1.0) - OutRejectionBlendFinal, tsr_tensor_half::Const(MinBlendFinal));
		
		FinalFlickeringHistory = InputLuma * FinalBlendFinal - FinalPrevFlickeringHistory * (FinalBlendFinal - tsr_tensor_half::Const(1.0));
		
		tsr_tensor_half GhostingHistory = InputLuma * tsr_tensor_half::Const(MinBlendFinal) - PrevFlickeringHistory * tsr_tensor_half::Const(MinBlendFinal - (1.0));

		CurrentGradient = FinalFlickeringHistory - GhostingHistory;
	}

	FinalFlickeringHistory = SMCSToGCS(FinalFlickeringHistory);

	// Remove how much luma has changed from luma measurement to final scene color from the gradient.
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_half LocalCurrentGradient = CurrentGradient.GetElement(ElementIndex);
		tsr_half LocalInputLumaModification = FilteredInputLumaModification.GetElement(ElementIndex);
			
		CurrentGradient.SetElement(ElementIndex, clamp(LocalCurrentGradient * tsr_half(POSITIVE_INFINITY), tsr_half(-1.0), tsr_half(1.0)) * max(abs(LocalCurrentGradient) - LocalInputLumaModification, tsr_half(0.0)));
	}

	#if 0
		tsr_tensor_half InputBoxSize = InputLumaMax - InputLumaMin;
		CurrentGradient = sign(CurrentGradient) * max(abs(CurrentGradient) - tsr_tensor_half::Const(0.5) * InputBoxSize, tsr_tensor_half::Const(0.0));
	#endif

	const tsr_tensor_half MoireEncodingErrors = tsr_tensor_half::Const(tsr_half(MoireEncodingError));
	tsr_tensor_bool bGradientsAreSameSign = CurrentGradient * PrevGradient > tsr_tensor_half::Const(0.0);
	tsr_tensor_bool bGradientsAreWithinError = or(abs(CurrentGradient) < MoireEncodingErrors, abs(PrevGradient) < MoireEncodingErrors);
	//bool bCurrentGradientSubstentialEnough = abs(LocalCurrentGradient) > View.GeneralPurposeTweak * abs(LocalPrevGradient); // - 2.0 * MoireEncodingError;
	
	#if CONFIG_THIN_GEOMETRY_DETECTION
	tsr_tensor_bool bGradientsAreThinGeometryRegion = tsr_tensor_bool::Const(false);
	if (bEnableThinGeometry)
	{
		tsr_tensor_bool bIsPotentialGeometryFlickering = HistoryRelaxationFactor > tsr_tensor_half::Const(0.0);
		tsr_tensor_bool bAnyGradient = or(abs(CurrentGradient) > ValidGeometryGradientThreshold,abs(PrevGradient) > ValidGeometryGradientThreshold);
		bGradientsAreThinGeometryRegion = and(and(bIsPotentialGeometryFlickering, bAnyGradient), bGradientsAreWithinError);
		bGradientsAreWithinError = select(bGradientsAreThinGeometryRegion, tsr_tensor_bool::Const(false), bGradientsAreWithinError);
	}
	#endif
	
	tsr_tensor_bool bCameraCuts = tsr_tensor_bool::Const(bool(bCameraCut));
	tsr_tensor_half IsFlicker = select(or(
			or(bGradientsAreSameSign,
			bGradientsAreWithinError),
			//!bCurrentGradientSubstentialEnough ||
			bCameraCuts), tsr_tensor_half::Const(0.0), tsr_tensor_half::Const(tsr_half(1.0)));

	// Compute the total variation of the gradient over time.
	tsr_tensor_half GradientVariation = min(abs(PrevGradient), abs(CurrentGradient)) * IsFlicker;

	// Dilates the total variation and contribution to keep stability despite inacuracies in the history reprojection
	// of these need a nearest filter which isn't precise at all.
	tsr_tensor_half TotalVariationContribution;
	tsr_tensor_half TotalVariationCountContribution;
	Deconcatenate(Max3x3(Max3x3(Concatenate(GradientVariation, IsFlicker))), /* out */ TotalVariationContribution, /* out */ TotalVariationCountContribution);

	// Blend out previous gradient
	tsr_tensor_half PrevBlendedGradient = PrevGradient * (MinBlendFinal - tsr_half(1.0)) * (IsFlicker - tsr_half(1.0));

	// Compute new gradient.
	#if 1
		tsr_tensor_half NewGradient = PrevBlendedGradient + CurrentGradient;
	#else
		tsr_tensor_half NewGradient = abs(PrevBlendedGradient) > abs(CurrentGradient) ? PrevBlendedGradient : CurrentGradient;
	#endif
	
	// Accumulate current frame variation.
	tsr_tensor_half NewTotalVariation =  PrevTotalVariation + TotalVariationContribution;
	tsr_tensor_half NewTotalVariationCount = PrevTotalVariationCount + TotalVariationCountContribution;

	// Discard all moire history on parallax disocclusion.
	NewGradient = select(bIsDisoccluded, tsr_tensor_half::Const(0.0), NewGradient);
	NewTotalVariation      = select(bIsDisoccluded, tsr_tensor_half::Const(0.0), NewTotalVariation);
	NewTotalVariationCount = select(bIsDisoccluded, tsr_tensor_half::Const(0.0), NewTotalVariationCount);

	// Quantise total variation and count to history bit depth to variation drift in the history over time.
	tsr_tensor_half QuantizedNewTotalVariationCount = floor(NewTotalVariationCount * (tsr_half(255.0) / MaxPrevTotalVariationCount)) * (MaxPrevTotalVariationCount / tsr_half(255.0));
	NewTotalVariation = NewTotalVariation * QuantizedNewTotalVariationCount * SafeRcp(NewTotalVariationCount);

	// Compute the final luminance
	#if CONFIG_THIN_GEOMETRY_DETECTION
		const tsr_tensor_half TotalVariationCountInvThresholdDefault = tsr_tensor_half::Const(tsr_half(TotalVariationCountInvThreshold[0]));
		const tsr_tensor_half TotalVariationCountInvThresholdThinGeo = tsr_tensor_half::Const(tsr_half(TotalVariationCountInvThreshold[1]));
		
		tsr_tensor_half TotalVariationCountInvThresholds = select(bGradientsAreThinGeometryRegion, TotalVariationCountInvThresholdThinGeo, TotalVariationCountInvThresholdDefault);
	#else
		tsr_tensor_half TotalVariationCountInvThresholds = tsr_tensor_half::Const(tsr_half(TotalVariationCountInvThreshold[0]));
	#endif

	#if 1
		tsr_tensor_half CountFadeIn = saturate(NewTotalVariationCount * TotalVariationCountInvThresholds - tsr_half(0.5));
	#else
		tsr_tensor_half CountFadeIn = saturate(NewTotalVariationCount - rcp(TotalVariationCountInvThresholds));
	#endif

	OutMoireError = (abs(NewTotalVariation * SafeRcp(NewTotalVariationCount)) + NewTotalVariationCount * MoireEncodingError) * CountFadeIn;
	
	OutMoireHistory = Concatenate( 
						Concatenate(FinalFlickeringHistory, NewGradient * tsr_half(127.0 / 255.0) + tsr_half(127.0 / 255.0)),
						Concatenate(NewTotalVariation, QuantizedNewTotalVariationCount * rcp(MaxPrevTotalVariationCount)));
	
	// Moving object in the scene have very low chance of creating moire pattern with TAA jitter, so discard the moire error to still
	// have character animation ghost free evem as environement shadow project different things on character.
	OutMoireError = saturate(OutMoireError * bIsStatic - InputLumaModification);
} // ComputeMoireError()


//------------------------------------------------------- SHADING REJECTION

void MeasureRejection(
	tsr_tensor_halfC InputC0,
	tsr_tensor_halfC InputC2,
	tsr_tensor_halfC HistoryC2,
	tsr_tensor_half  MoireError,
#if CONFIG_THIN_GEOMETRY_DETECTION
	tsr_tensor_half HistoryRelaxationFactor,
	const bool bEnableHistoryControl,
#endif
	bool bEnableMoireError,
	out tsr_tensor_half OutRejectionBlendFinal,
	out tsr_tensor_half OutRejectionClampBlend)
{
	tsr_tensor_halfC BackbufferQuantizationErrorVector = tsr_tensor_halfC::Const(MeasureBackbufferLDRQuantizationError());
		
	tsr_tensor_halfC FilteredHistory = Blur3x3(HistoryC2);
	
	tsr_tensor_halfC TotalVarInputDiffC0C2 = abs(TotalVariation3x3(abs(InputC0 - InputC2)));
	tsr_tensor_halfC TotalVarInputC2 = abs(TotalVariation3x3(InputC2));

	tsr_tensor_halfC ClampError = BackbufferQuantizationErrorVector;
	ClampError = max(ClampError, Blur3x3(min(TotalVarInputDiffC0C2, TotalVarInputC2)));
	
	tsr_tensor_halfC InputC2BoxSize = MaxMinusMin3x3(InputC2);
	ClampError = max(ClampError, InputC2BoxSize * tsr_half(FilteringWeight * 0.25));
	ClampError = ClampError + BackbufferQuantizationErrorVector;

	tsr_tensor_halfC FilteredInput   = Blur3x3(InputC2);

	// Build the clamping box of the filtered signal
	tsr_tensor_halfC FilteredBoxMin;
	tsr_tensor_halfC FilteredBoxMax;
	MinMax3x3(FilteredInput, /* out */ FilteredBoxMin, /* out */ FilteredBoxMax);

#if CONFIG_THIN_GEOMETRY_DETECTION  
	if (bEnableHistoryControl)
	{
		// VGPR optimization for LerpClamp3x3()
		HistoryRelaxationFactor = select(HistoryRelaxationFactor > tsr_tensor_half::Const(tsr_half(1.0f / 127.0f)), HistoryRelaxationFactor, tsr_tensor_half::Const(0));
		tsr_tensor_halfC LerpFactorVector = tsr_tensor_halfC::Vectorize((HistoryRelaxationFactor));
		
		PLATFORM_SPECIFIC_ISOLATE
		{
			// Register pressure is high. No MinMax3x3()
			tsr_tensor_halfC ToClampBoundaryLimit;
				
			ToClampBoundaryLimit = Min3x3(FilteredHistory);
			FilteredBoxMin = lerp(FilteredBoxMin, ToClampBoundaryLimit, LerpFactorVector);

			ToClampBoundaryLimit = Max3x3(FilteredHistory);
			FilteredBoxMax = lerp(FilteredBoxMax, ToClampBoundaryLimit, LerpFactorVector);
		}
		
		tsr_tensor_halfC LerpClampedHistoryInput = fastClamp(FilteredHistory, FilteredBoxMin, FilteredBoxMax);

		{
			// Register pressure is high. No MinMax3x3()
			tsr_tensor_halfC ToClampBoundaryLimit;
				
			ToClampBoundaryLimit = Min3x3(LerpClampedHistoryInput);
			FilteredBoxMin = select(LerpFactorVector > tsr_tensor_halfC::Const(0), ToClampBoundaryLimit, FilteredBoxMin);

			ToClampBoundaryLimit = Max3x3(LerpClampedHistoryInput);
			FilteredBoxMax = select(LerpFactorVector > tsr_tensor_halfC::Const(0), ToClampBoundaryLimit, FilteredBoxMax);
		}
	}
#endif

	FilteredBoxMin = FilteredBoxMin - ClampError;
	FilteredBoxMax = FilteredBoxMax + ClampError;
		
	// Clamp filtered signal
	tsr_tensor_halfC ClampedFilteredHistory = fastClamp(FilteredHistory, FilteredBoxMin, FilteredBoxMax);
	BRANCH
	if (bEnableMoireError)
	{
		tsr_tensor_halfC MoireErrorSize;
			
		UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		#if 0
		{
			tsr_halfC ClampedEnergy = abs(FlickeringClampedFilteredHistory.GetElement(ElementIndex) - FilteredHistory.GetElement(ElementIndex));
			tsr_halfC ClampedEnergyAmount = ClampedEnergy * SafeRcp(ClampedEnergy[0] + ClampedEnergy[1] + ClampedEnergy[2]);

			// Apply the moire error on each individual channel based on how much they need to clamp.
			MoireErrorSize.SetElement(ElementIndex, ClampedEnergyAmount * (kMoireLumaToChannel * MoireError.GetElement(ElementIndex)));
		}
		#else
		{
			MoireErrorSize.SetElement(ElementIndex, kMoireLumaToChannel * MoireError.GetElement(ElementIndex));
		}
		#endif

		tsr_tensor_halfC StableFilteredBoxMin = min(FilteredBoxMin, FilteredBoxMax - MoireErrorSize);
		tsr_tensor_halfC StableFilteredBoxMax = max(FilteredBoxMax, FilteredBoxMin + MoireErrorSize);
			
		ClampedFilteredHistory = fastClamp(FilteredHistory, StableFilteredBoxMin, StableFilteredBoxMax);
	}
		
	// Compute the box size.
	tsr_tensor_halfC BoxSize;
	{
		BoxSize = (
			InputC2BoxSize * tsr_tensor_halfC::Const(FilteringWeight) +
			(BackbufferQuantizationErrorVector * tsr_half(2 * FilteringWeight)));

		BRANCH
		if (bEnableMoireError)
		{
			tsr_tensor_halfC MoireErrorSize;
			UNROLL_N(SIMD_SIZE)
			for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
			{
				tsr_halfC ClampedEnergy = abs(ClampedFilteredHistory.GetElement(ElementIndex) - FilteredHistory.GetElement(ElementIndex));
				tsr_halfC ClampedEnergyAmount = ClampedEnergy * SafeRcp(ClampedEnergy[0] + ClampedEnergy[1] + ClampedEnergy[2]);

				// Apply the moire error on each individual channel based on how much they need to clamp.
				MoireErrorSize.SetElement(ElementIndex, ClampedEnergyAmount * ((FilteringWeight * tsr_half(3.0)) * MoireError.GetElement(ElementIndex)));
			}
			BoxSize = max(BoxSize, MoireErrorSize);
		}
	}

	tsr_tensor_halfC Delta = max(abs(FilteredInput - FilteredHistory), BoxSize);

	tsr_tensor_halfC RawClampedEnergy      = abs(ClampedFilteredHistory - FilteredHistory);

	tsr_tensor_halfC RawFactor        = saturate(tsr_tensor_halfC::Const(1.0) - RawClampedEnergy * rcp(Delta));
	tsr_tensor_half RawRejection      = min3(RawFactor[0], RawFactor[1], RawFactor[2]);
	#if CONFIG_SCENE_COLOR_ALPHA
		RawRejection = min(RawRejection, RawFactor[3]);
	#endif

	#if CONFIG_SCENE_COLOR_ALPHA
		OutRejectionClampBlend = Min3x3(Median3x3(RawRejection));
		tsr_tensor_halfC FilteredClampedEnergy = Max3x3(Median3x3(RawClampedEnergy));
	#else
		tsr_tensor_halfC FilteredClampedEnergy;
		Deconcatenate(MaxRGBMinA3x3(Median3x3(Concatenate(RawClampedEnergy, RawRejection))), /* out */ FilteredClampedEnergy, /* out */ OutRejectionClampBlend);
	#endif

	tsr_tensor_halfC FilteredFactor   = saturate(tsr_tensor_halfC::Const(1.0) - FilteredClampedEnergy * rcp(Delta));
	tsr_tensor_half FilteredRejection = min3(FilteredFactor[0], FilteredFactor[1], FilteredFactor[2]);
	#if CONFIG_SCENE_COLOR_ALPHA
		FilteredRejection = min(FilteredRejection, FilteredFactor[3]);
	#endif
	OutRejectionBlendFinal          = FilteredRejection;
} // MeasureRejection()
