// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRSpatialAntiAliasing.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#if DIM_QUALITY_PRESET == 1
	#define CONFIG_ITERATIONS 3

#elif DIM_QUALITY_PRESET == 2
	#define CONFIG_ITERATIONS 8

#else
	#error unknown DIM_QUALITY_PRESET
#endif

#if CONFIG_COMPILE_FP16
	// Take advantage of RDNA's v_pk_*_{uif}16 instructions
	#define CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION 1
#else
	#define CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION 0
#endif


//------------------------------------------------------- PARAMETERS

Texture2D<tsr_ushort> AntiAliasMaskTexture;
Texture2D<tsr_half>  InputSceneColorLdrLumaTexture;

RWTexture2D<tsr_ushort2> AntiAliasingOutput;


//------------------------------------------------------- FUNCTIONS

tsr_ushort2x2 Map8x8Tile2x2LaneDPV(uint GroupThreadIndex)
#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
{
	tsr_ushort T = tsr_ushort(GroupThreadIndex);

	tsr_ushort2 GroupId;
	GroupId.x =                                             ((T >> tsr_ushort(1 - 1)) & tsr_ushort(0x03 << 1));
	GroupId.y = ((T >> tsr_ushort(0)) & tsr_ushort(0x01)) | ((T >> tsr_ushort(3 - 1)) & tsr_ushort(0x03 << 1));
	
	return dpv_interleave_registers(GroupId, GroupId + tsr_ushort2(1, 0));
}
#else
{
	tsr_ushort2 GroupId = Map8x8Tile2x2Lane(GroupThreadIndex);
	return dpv_interleave_mono_registers(GroupId);
}
#endif

tsr_half SampleInputLDRLuma(tsr_short2 KernelCenter, tsr_short2 Offset)
{
	tsr_short2 SampleInputPixelPos = KernelCenter + ClampPixelOffset(
		KernelCenter,
		Offset, Offset,
		InputPixelPosMin, InputPixelPosMax);
	
	return InputSceneColorLdrLumaTexture[SampleInputPixelPos];
}

tsr_half2 SampleInputLDRLuma(tsr_short2x2 KernelCenter, tsr_short2 Offset)
{
	tsr_short2x2 SampleInputPixelPos = KernelCenter + ClampPixelOffset(
		KernelCenter,
		dpv_interleave_mono_registers(Offset), Offset,
		InputPixelPosMin, InputPixelPosMax);
	
	tsr_half2 InputLuma = dpv_interleave_registers(
		InputSceneColorLdrLumaTexture[dpv_lo(SampleInputPixelPos)],
		InputSceneColorLdrLumaTexture[dpv_hi(SampleInputPixelPos)]);

	return InputLuma;
}


//------------------------------------------------------- ENTRY POINT

#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
	[numthreads(TILE_SIZE * TILE_SIZE / 2, 1, 1)]
#else
	[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
#endif
void SpatialAntiAliasingCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint GroupWaveIndex = GetGroupWaveIndex(GroupThreadIndex, /* GroupSize = */ TILE_SIZE * TILE_SIZE);

	float4x2 Debug = 0.0;
	
	tsr_short2x2 InputPixelPos = dpv_add(
		tsr_short2(InputPixelPosMin) + tsr_short2(GroupId) * tsr_short2(TILE_SIZE, TILE_SIZE),
		tsr_short2x2(Map8x8Tile2x2LaneDPV(GroupThreadIndex)));
	
	//Debug[0] = 1.0;

	bool2 bAntiAlias;
	bAntiAlias[0] = AntiAliasMaskTexture[dpv_lo(InputPixelPos)] > tsr_ushort(0);
	bAntiAlias[1] = AntiAliasMaskTexture[dpv_hi(InputPixelPos)] > tsr_ushort(0);
	
	// Whether it is worth to anti-aliasing.
	bool bSkipAntiAliaser = all(!bAntiAlias);

	tsr_half2x2 TexelOffset;
	tsr_half2 NoiseFiltering;

	BRANCH
	if (bSkipAntiAliaser)
	{
		TexelOffset = tsr_half(0.0);
		NoiseFiltering = tsr_half(0.0);
	}
	else
	{
		#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION && 1
			tsr_half InputCache[3 * 4];
			UNROLL_N(3)
			for (int cy = 0; cy < 3; cy++)
			{
				UNROLL_N(4)
				for (int cx = 0; cx < 4; cx++)
				{
					const uint ci = cx + 4 * cy;
					InputCache[ci] = SampleInputLDRLuma(dpv_lo(InputPixelPos), tsr_short2(cx - 1, cy - 1));
				}
			}

			#define ReadCache(x, y) tsr_half2(InputCache[(x + 1) + 4 * (y + 1)], InputCache[(x + 2) + 4 * (y + 1)])
		#else
			#define ReadCache(x, y) SampleInputLDRLuma(InputPixelPos, tsr_short2(x, y));
		#endif
		
		tsr_half2 InputC = ReadCache( 0,  0);

		tsr_half2 InputN = ReadCache( 0, -1);
		tsr_half2 InputS = ReadCache( 0, +1);
		tsr_half2 InputE = ReadCache(+1,  0);
		tsr_half2 InputW = ReadCache(-1,  0);
	
		tsr_half2 InputNE = ReadCache(+1, -1);
		tsr_half2 InputNW = ReadCache(-1, -1);
		tsr_half2 InputSE = ReadCache(+1, +1);
		tsr_half2 InputSW = ReadCache(-1, +1);

		tsr_short2x2 BrowseDirection;
		tsr_short2x2 EdgeSide;
		tsr_half2 EdgeLuma;
		FindBrowsingDirection(
			InputC,
			InputN, InputS, InputE, InputW,
			InputNE, InputNW, InputSE, InputSW,
			/* out */ NoiseFiltering,
			/* out */ BrowseDirection,
			/* out */ EdgeSide,
			/* out */ EdgeLuma);

		tsr_half2 LumaDelta = abs(EdgeLuma - InputC) * 0.5;
		bool2 WorthBrowsing = LumaDelta > SPATIAL_ANTI_ALIASER_MIN_LUMIMANCE;
		tsr_half2 BrowseDelta = saturate(LumaDelta * rcp(tsr_half(SPATIAL_ANTI_ALIASER_MIN_LUMIMANCE)));
		
		tsr_ushort2 EdgeLengthP, EdgeLengthN;
		bool2 bEdgeStopedByIncrementP, bEdgeStopedByIncrementN;
		bool2 bEdgeStopedByDecrementP, bEdgeStopedByDecrementN;
		
		BrowseNeighborhoodBilinearOptimized(
			/* Iterations = */ CONFIG_ITERATIONS,
			/* Texture = */ InputSceneColorLdrLumaTexture,
			InputC,
			EdgeLuma,
			InputPixelPos,
			BrowseDirection,
			EdgeSide,
			/* out */ EdgeLengthP,
			/* out */ EdgeLengthN,
			/* out */ bEdgeStopedByIncrementP,
			/* out */ bEdgeStopedByIncrementN,
			/* out */ bEdgeStopedByDecrementP,
			/* out */ bEdgeStopedByDecrementN);

		tsr_half TexelOffsetLo = ComputeDistanceToEdge(
			bEdgeStopedByIncrementN[0],
			bEdgeStopedByIncrementP[0],
			bEdgeStopedByDecrementN[0],
			bEdgeStopedByDecrementP[0],
			dpv_lo(EdgeLengthN),
			dpv_lo(EdgeLengthP));
		
		tsr_half TexelOffsetHi = ComputeDistanceToEdge(
			bEdgeStopedByIncrementN[1],
			bEdgeStopedByIncrementP[1],
			bEdgeStopedByDecrementN[1],
			bEdgeStopedByDecrementP[1],
			dpv_hi(EdgeLengthN),
			dpv_hi(EdgeLengthP));
		
		FLATTEN
		if (!WorthBrowsing[0])
			TexelOffsetLo = 0.0;
		
		FLATTEN
		if (!WorthBrowsing[1])
			TexelOffsetHi = 0.0;

		TexelOffset = dpv_scale(tsr_half2x2(EdgeSide), BrowseDelta * dpv_interleave_registers(TexelOffsetLo, TexelOffsetHi));
	}

	uint2 EncodedTexelOffset = EncodeSpatialAntiAliasingOffset(TexelOffset);

	{
		tsr_short2x2 OutputPixelPos = InputPixelPos;
		
		OutputPixelPos = InvalidateOutputPixelPos(OutputPixelPos, InputInfo_ViewportMax);
		
		AntiAliasingOutput[dpv_lo(OutputPixelPos)] = tsr_ushort2(dpv_lo(EncodedTexelOffset), round(dpv_lo(NoiseFiltering) * tsr_half(255.0)));
		
		#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
			AntiAliasingOutput[dpv_hi(OutputPixelPos)] = tsr_ushort2(dpv_hi(EncodedTexelOffset), round(dpv_hi(NoiseFiltering) * tsr_half(255.0)));
		#endif

		#if DEBUG_OUTPUT
		{
			DebugOutput[tsr_short3(dpv_lo(OutputPixelPos), 0)] = dpv_lo(Debug);

			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				DebugOutput[tsr_short3(dpv_hi(OutputPixelPos), 0)] = dpv_hi(Debug);
			#endif
		}
		#endif
	}
}
