// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TSRKernels.ush"


//------------------------------------------------------- DEFINITIONS

#define SPATIAL_ANTI_ALIASER_MIN_LUMIMANCE 0.05


//------------------------------------------------------- SAMPLING FUNCTION

tsr_half2 SampleAAInput(Texture2D<tsr_half> Texture, tsr_short2x2 KernelCenter, tsr_short2x2 Offset)
{
	tsr_short2x2 SampleInputPixelPos = ClampPixelOffset(
		KernelCenter + Offset,
		InputPixelPosMin, InputPixelPosMax);
	
	tsr_half2 InputLuma = dpv_interleave_registers(
		Texture[dpv_lo(SampleInputPixelPos)],
		Texture[dpv_hi(SampleInputPixelPos)]);

	return InputLuma;
}

float BilinearSampleAAInput(Texture2D<float> Texture, float2 KernelCenterUV, float2 UVOffset, const bool bIsNegativeDirection)
{
	float2 SampleUV = KernelCenterUV + UVOffset;

	if (bIsNegativeDirection)
		SampleUV = max(SampleUV, InputInfo_UVViewportBilinearMin);
	else
		SampleUV = min(SampleUV, InputInfo_UVViewportBilinearMax);

	return Texture.SampleLevel(GlobalBilinearClampedSampler, SampleUV, 0);
}

tsr_half2 BilinearSampleAAInput(Texture2D<tsr_half> Texture, float2x2 KernelCenterUV, float2x2 UVOffset, const bool bIsNegativeDirection)
{
	float2x2 SampleUV = KernelCenterUV + UVOffset;

	if (bIsNegativeDirection)
		SampleUV = max(SampleUV, dpv_interleave_mono_registers(InputInfo_UVViewportBilinearMin));
	else
		SampleUV = min(SampleUV, dpv_interleave_mono_registers(InputInfo_UVViewportBilinearMax));

	tsr_half2 InputLuma = dpv_interleave_registers(
		Texture.SampleLevel(GlobalBilinearClampedSampler, dpv_lo(SampleUV), 0),
		Texture.SampleLevel(GlobalBilinearClampedSampler, dpv_hi(SampleUV), 0));

	return InputLuma;
}


//------------------------------------------------------- FUNCTIONS

/** Finds the browsing direction of the spatial anti-aliaser out from the neighborhood. */
void FindBrowsingDirection(
	float InputC,
	float InputN,
	float InputS,
	float InputE,
	float InputW,
	float InputNE,
	float InputNW,
	float InputSE,
	float InputSW,
	out float NoiseFiltering,
	out tsr_short2 BrowseDirection,
	out tsr_short2 EdgeSide,
	out float EdgeInput)
{
	float TotalVarHN = abs(0.5 * (InputNE + InputNW) - InputN);
	float TotalVarH  = abs(0.5 * (InputE  + InputW ) - InputC);
	float TotalVarHS = abs(0.5 * (InputSE + InputSW) - InputS);
			
	float TotalVarVE = abs(0.5 * (InputNE + InputSE) - InputE);
	float TotalVarV  = abs(0.5 * (InputN  + InputS ) - InputC);
	float TotalVarVW = abs(0.5 * (InputNW + InputSW) - InputW);
		
	float DiffN = abs(InputN - InputC);
	float DiffS = abs(InputS - InputC);
	float DiffE = abs(InputE - InputC);
	float DiffW = abs(InputW - InputC);

	bool bBrowseVerticaly = (TotalVarHN + TotalVarH + TotalVarHS) > (TotalVarVE + TotalVarV + TotalVarVW);
	BrowseDirection[0] = select(bBrowseVerticaly, tsr_short(0), tsr_short(1));
	BrowseDirection[1] = select(bBrowseVerticaly, tsr_short(1), tsr_short(0));
	
	{
		NoiseFiltering = saturate(float(2.0) * TotalVarH - max(DiffE, DiffW));
		NoiseFiltering = max(NoiseFiltering, saturate(float(2.0) * TotalVarV - max(DiffN, DiffS)));
	}

	EdgeSide = tsr_short(0);

	FLATTEN
	if (bBrowseVerticaly)
	{
		EdgeSide[0] = DiffW > DiffE ? -1 : 1;
		EdgeInput = DiffW > DiffE ? InputW : InputE;
	}
	else
	{
		EdgeSide[1] = DiffN > DiffS ? -1 : 1;
		EdgeInput = DiffN > DiffS ? InputN : InputS;
	}
} // FindBrowsingDirection()

/** Finds the browsing direction of the spatial anti-aliaser out from the neighborhood. */
void FindBrowsingDirection(
	tsr_half2 InputC,
	tsr_half2 InputN,
	tsr_half2 InputS,
	tsr_half2 InputE,
	tsr_half2 InputW,
	tsr_half2 InputNE,
	tsr_half2 InputNW,
	tsr_half2 InputSE,
	tsr_half2 InputSW,
	out tsr_half2 NoiseFiltering,
	out tsr_short2x2 BrowseDirection,
	out tsr_short2x2 EdgeSide,
	out tsr_half2 EdgeLuma)
{
	tsr_half2 TotalVarHN = abs(0.5 * (InputNE + InputNW) - InputN);
	tsr_half2 TotalVarH  = abs(0.5 * (InputE  + InputW ) - InputC);
	tsr_half2 TotalVarHS = abs(0.5 * (InputSE + InputSW) - InputS);
			
	tsr_half2 TotalVarVE = abs(0.5 * (InputNE + InputSE) - InputE);
	tsr_half2 TotalVarV  = abs(0.5 * (InputN  + InputS ) - InputC);
	tsr_half2 TotalVarVW = abs(0.5 * (InputNW + InputSW) - InputW);
		
	tsr_half2 DiffN = abs(InputN - InputC);
	tsr_half2 DiffS = abs(InputS - InputC);
	tsr_half2 DiffE = abs(InputE - InputC);
	tsr_half2 DiffW = abs(InputW - InputC);

	bool2 bBrowseVerticaly = (TotalVarHN + TotalVarH + TotalVarHS) > (TotalVarVE + TotalVarV + TotalVarVW);
	BrowseDirection[0] = select(bBrowseVerticaly, tsr_short(0), tsr_short(1));
	BrowseDirection[1] = select(bBrowseVerticaly, tsr_short(1), tsr_short(0));
	
	{
		NoiseFiltering = saturate(tsr_half(2.0) * TotalVarH - max(DiffE, DiffW));
		NoiseFiltering = max(NoiseFiltering, saturate(tsr_half(2.0) * TotalVarV - max(DiffN, DiffS)));
	}

	EdgeSide = tsr_short(0);

	FLATTEN
	if (bBrowseVerticaly[0])
	{
		EdgeSide[0][0] = DiffW[0] > DiffE[0] ? -1 : 1;
		EdgeLuma[0] = DiffW[0] > DiffE[0] ? InputW[0] : InputE[0];
	}
	else
	{
		EdgeSide[1][0] = DiffN[0] > DiffS[0] ? -1 : 1;
		EdgeLuma[0] = DiffN[0] > DiffS[0] ? InputN[0] : InputS[0];
	}
		
	FLATTEN
	if (bBrowseVerticaly[1])
	{
		EdgeSide[0][1] = DiffW[1] > DiffE[1] ? -1 : 1;
		EdgeLuma[1] = DiffW[1] > DiffE[1] ? InputW[1] : InputE[1];
	}
	else
	{
		EdgeSide[1][1] = DiffN[1] > DiffS[1] ? -1 : 1;
		EdgeLuma[1] = DiffN[1] > DiffS[1] ? InputN[1] : InputS[1];
	}
} // FindBrowsingDirection()


//------------------------------------------------------- ACCUMULATE

/** Accumulate an edge length */
void AccumulateEdgeLength(
	tsr_half2 StartLuma, tsr_half2 StartEdgeLuma,
	tsr_half2 SampleLuma, tsr_half2 SampleEdgeLuma,
	tsr_half2 LumaDelta,
	inout bool2 bEdgeStopedByIncrement,
	inout bool2 bEdgeStopedByDecrement,
	inout tsr_ushort2 EdgeLength)
{
	bool2 bStopNowIncrement = abs(StartLuma - SampleLuma) > LumaDelta;
	bool2 bStopNowDecrement = abs(StartEdgeLuma - SampleEdgeLuma) > LumaDelta;
	
	/**
	 * Increment:
	 *     1 1 1 1 1
	 *     0 0 X 0 1
	 * 
	 * Decrement:
	 *     1 1 1 1 0
	 *     0 0 X 0 0
	 */
	bEdgeStopedByIncrement = or(bEdgeStopedByIncrement, bStopNowIncrement);
	bEdgeStopedByDecrement = or(bEdgeStopedByDecrement, bStopNowDecrement);

	EdgeLength += select(or(bEdgeStopedByIncrement, bEdgeStopedByDecrement), tsr_ushort(0), tsr_ushort(1));
}

void BrowseNeighborhood(
	const uint Iterations,
	Texture2D<tsr_half> Texture,
	tsr_half2 InputC,
	tsr_half2 EdgeLuma,
	tsr_half2 LumaDelta,
	tsr_short2x2 KernelCenter,
	tsr_short2x2 BrowseDirection,
	tsr_short2x2 EdgeSide,
	out tsr_ushort2 EdgeLengthP,
	out tsr_ushort2 EdgeLengthN,
	out bool2 bEdgeStopedByIncrementP,
	out bool2 bEdgeStopedByIncrementN,
	out bool2 bEdgeStopedByDecrementP,
	out bool2 bEdgeStopedByDecrementN)
{
	EdgeLengthP = 0;
	EdgeLengthN = 0;
	bEdgeStopedByIncrementP = false;
	bEdgeStopedByIncrementN = false;
	bEdgeStopedByDecrementP = false;
	bEdgeStopedByDecrementN = false;
	
	UNROLL
	for (uint i = 0; i < Iterations; i++)
	{
		bool2 bIsStoped = and(or(bEdgeStopedByIncrementP, bEdgeStopedByDecrementP), or(bEdgeStopedByIncrementN, bEdgeStopedByDecrementP));

		BRANCH
		if (!bIsStoped.x)
		{
			tsr_short2x2 NeighborOffset = BrowseDirection * tsr_short(i + 1);

			tsr_half2 NeighborP = SampleAAInput(Texture, KernelCenter, NeighborOffset);
			tsr_half2 EdgeP     = SampleAAInput(Texture, KernelCenter, NeighborOffset + EdgeSide);
			
			tsr_half2 NeighborN = SampleAAInput(Texture, KernelCenter, -NeighborOffset);
			tsr_half2 EdgeN     = SampleAAInput(Texture, KernelCenter, -NeighborOffset + EdgeSide);

			AccumulateEdgeLength(
				InputC, EdgeLuma,
				NeighborP, EdgeP,
				LumaDelta,
				/* inout */ bEdgeStopedByIncrementP,
				/* inout */ bEdgeStopedByDecrementP,
				/* inout */ EdgeLengthP);
			
			AccumulateEdgeLength(
				InputC, EdgeLuma,
				NeighborN, EdgeN,
				LumaDelta,
				/* inout */ bEdgeStopedByIncrementN,
				/* inout */ bEdgeStopedByDecrementN,
				/* inout */ EdgeLengthN);
		}
	}
} // BrowseNeighborhood()

void BrowseNeighborhoodBilinearOptimized(
	const uint Iterations,
	Texture2D<tsr_half> Texture,
	tsr_half2 InputC,
	tsr_half2 EdgeLuma,
	tsr_short2x2 InputPixelPos,
	tsr_short2x2 BrowseDirection,
	tsr_short2x2 EdgeSide,
	out tsr_ushort2 EdgeLengthP,
	out tsr_ushort2 EdgeLengthN,
	out bool2 bEdgeStopedByIncrementP,
	out bool2 bEdgeStopedByIncrementN,
	out bool2 bEdgeStopedByDecrementP,
	out bool2 bEdgeStopedByDecrementN)
{
	EdgeLengthP = 0;
	EdgeLengthN = 0;
	bEdgeStopedByIncrementP = false;
	bEdgeStopedByIncrementN = false;
	bEdgeStopedByDecrementP = false;
	bEdgeStopedByDecrementN = false;

	const tsr_half BilinearInterp = 0.5;

	float2x2 fBrowseDirection = float2x2(BrowseDirection) * dpv_interleave_mono_registers(InputInfo_ExtentInverse);
	float2x2 KernelUV = (float2x2(InputPixelPos) + 0.5 + float2x2(EdgeSide) * BilinearInterp) * dpv_interleave_mono_registers(InputInfo_ExtentInverse);

	tsr_half2 MergedEdgeLuma = BilinearInterp * (EdgeLuma + InputC);
		
	tsr_half2 MinLuma = MergedEdgeLuma - tsr_half(BilinearInterp * 0.5) * abs(EdgeLuma - InputC);
	tsr_half2 MaxLuma = MergedEdgeLuma + tsr_half(BilinearInterp * 0.5) * abs(EdgeLuma - InputC);

	bool2 bEdgeMoreLuminous = EdgeLuma > InputC;
		
	tsr_ushort2 EdgeStopedByMinLumaP = 0;
	tsr_ushort2 EdgeStopedByMinLumaN = 0;
		
	tsr_ushort2 EdgeStopedByMaxLumaP = 0;
	tsr_ushort2 EdgeStopedByMaxLumaN = 0;
		
	bool bStop = false;
		
	#if CONFIG_COMPILE_FP16
		#define asuintX(x) asuint16(x)
		const tsr_ushort SignShift = 15;
		const tsr_ushort SignMask = 0x8000;
	#else
		#define asuintX(x) asuint(x)
		const tsr_ushort SignShift = 31;
		const tsr_ushort SignMask = 0x80000000;
	#endif
			
	EdgeLengthP = tsr_ushort(Iterations);
	EdgeLengthN = tsr_ushort(Iterations);
		
	UNROLL
	for (uint i = 0; i < Iterations; i++)
	{
		float2x2 NeighborOffset = fBrowseDirection * float(i + 1);

		tsr_half2 SampleLumaP = BilinearSampleAAInput(Texture, KernelUV,  NeighborOffset, /* bIsNegativeDirection = */ false);
		tsr_half2 SampleLumaN = BilinearSampleAAInput(Texture, KernelUV, -NeighborOffset, /* bIsNegativeDirection = */ true);
				
		//bEdgeStopedByMinLuma = bEdgeStopedByMinLuma || bool2(SampleLuma < MinLuma) && !bEdgeStopedByMaxLuma;
		//bEdgeStopedByMaxLuma = bEdgeStopedByMaxLuma || bool2(SampleLuma > MaxLuma) && !bEdgeStopedByMinLuma;
		tsr_ushort2 StopByMinLumaP = asuintX(SampleLumaP - MinLuma) & ~EdgeStopedByMaxLumaP;
		tsr_ushort2 StopByMaxLumaP = asuintX(MaxLuma - SampleLumaP) & ~EdgeStopedByMinLumaP;
		tsr_ushort2 StopByMinLumaN = asuintX(SampleLumaN - MinLuma) & ~EdgeStopedByMaxLumaN;
		tsr_ushort2 StopByMaxLumaN = asuintX(MaxLuma - SampleLumaN) & ~EdgeStopedByMinLumaN;

		EdgeStopedByMinLumaP |= StopByMinLumaP;
		EdgeStopedByMaxLumaP |= StopByMaxLumaP;
		EdgeStopedByMinLumaN |= StopByMinLumaN;
		EdgeStopedByMaxLumaN |= StopByMaxLumaN;
				
		//EdgeLength += (bEdgeStopedByMaxLuma || bEdgeStopedByMinLuma) ? 0 : 1;
		EdgeLengthP -= (EdgeStopedByMaxLumaP | EdgeStopedByMinLumaP) >> SignShift;
		EdgeLengthN -= (EdgeStopedByMaxLumaN | EdgeStopedByMinLumaN) >> SignShift;
	}
		
	bool2 bEdgeStopedByMinLumaP = (EdgeStopedByMinLumaP & SignMask) != 0;
	bool2 bEdgeStopedByMinLumaN = (EdgeStopedByMinLumaN & SignMask) != 0;
		
	bool2 bEdgeStopedByMaxLumaP = (EdgeStopedByMaxLumaP & SignMask) != 0;
	bool2 bEdgeStopedByMaxLumaN = (EdgeStopedByMaxLumaN & SignMask) != 0;
		
	bEdgeStopedByIncrementP = or(and(bEdgeMoreLuminous, bEdgeStopedByMaxLumaP), and(!bEdgeMoreLuminous, bEdgeStopedByMinLumaP));
	bEdgeStopedByIncrementN = or(and(bEdgeMoreLuminous, bEdgeStopedByMaxLumaN), and(!bEdgeMoreLuminous, bEdgeStopedByMinLumaN));
		
	bEdgeStopedByDecrementP = or(and(bEdgeMoreLuminous, bEdgeStopedByMinLumaP), and(!bEdgeMoreLuminous, bEdgeStopedByMaxLumaP));
	bEdgeStopedByDecrementN = or(and(bEdgeMoreLuminous, bEdgeStopedByMinLumaN), and(!bEdgeMoreLuminous, bEdgeStopedByMaxLumaN));


} // BrowseNeighborhoodBilinearRDNAOptimized

void BrowseNeighborhoodBilinearOptimized(
	const uint Iterations,
	Texture2D<float> Texture,
	float InputC,
	float EdgeLuma,
	tsr_short2 InputPixelPos,
	tsr_short2 BrowseDirection,
	tsr_short2 EdgeSide,
	out tsr_ushort EdgeLengthP,
	out tsr_ushort EdgeLengthN,
	out bool bEdgeStopedByIncrementP,
	out bool bEdgeStopedByIncrementN,
	out bool bEdgeStopedByDecrementP,
	out bool bEdgeStopedByDecrementN)
{
	EdgeLengthP = 0;
	EdgeLengthN = 0;
	bEdgeStopedByIncrementP = false;
	bEdgeStopedByIncrementN = false;
	bEdgeStopedByDecrementP = false;
	bEdgeStopedByDecrementN = false;

	const float BilinearInterp = 0.5;

	float2 fBrowseDirection = float2(BrowseDirection) * InputInfo_ExtentInverse;
	float2 KernelUV = (float2(InputPixelPos) + 0.5 + float2(EdgeSide) * BilinearInterp) * InputInfo_ExtentInverse;

	float MergedEdgeLuma = BilinearInterp * (EdgeLuma + InputC);
		
	float MinLuma = MergedEdgeLuma - float(BilinearInterp * 0.5) * abs(EdgeLuma - InputC);
	float MaxLuma = MergedEdgeLuma + float(BilinearInterp * 0.5) * abs(EdgeLuma - InputC);

	bool bEdgeMoreLuminous = EdgeLuma > InputC;
		
	uint EdgeStopedByMinLumaP = 0;
	uint EdgeStopedByMinLumaN = 0;
		
	uint EdgeStopedByMaxLumaP = 0;
	uint EdgeStopedByMaxLumaN = 0;
		
	bool bStop = false;
		
	const uint SignShift = 31;
	const uint SignMask = 0x80000000;
			
	EdgeLengthP = tsr_ushort(Iterations);
	EdgeLengthN = tsr_ushort(Iterations);
		
	UNROLL
	for (uint i = 0; i < Iterations; i++)
	{
		float2 NeighborOffset = fBrowseDirection * float(i + 1);

		float SampleLumaP = BilinearSampleAAInput(Texture, KernelUV,  NeighborOffset, /* bIsNegativeDirection = */ false);
		float SampleLumaN = BilinearSampleAAInput(Texture, KernelUV, -NeighborOffset, /* bIsNegativeDirection = */ true);
				
		//bEdgeStopedByMinLuma = bEdgeStopedByMinLuma || bool2(SampleLuma < MinLuma) && !bEdgeStopedByMaxLuma;
		//bEdgeStopedByMaxLuma = bEdgeStopedByMaxLuma || bool2(SampleLuma > MaxLuma) && !bEdgeStopedByMinLuma;
		uint StopByMinLumaP = asuint(SampleLumaP - MinLuma) & ~EdgeStopedByMaxLumaP;
		uint StopByMaxLumaP = asuint(MaxLuma - SampleLumaP) & ~EdgeStopedByMinLumaP;
		uint StopByMinLumaN = asuint(SampleLumaN - MinLuma) & ~EdgeStopedByMaxLumaN;
		uint StopByMaxLumaN = asuint(MaxLuma - SampleLumaN) & ~EdgeStopedByMinLumaN;

		EdgeStopedByMinLumaP |= StopByMinLumaP;
		EdgeStopedByMaxLumaP |= StopByMaxLumaP;
		EdgeStopedByMinLumaN |= StopByMinLumaN;
		EdgeStopedByMaxLumaN |= StopByMaxLumaN;
				
		//EdgeLength += (bEdgeStopedByMaxLuma || bEdgeStopedByMinLuma) ? 0 : 1;
		EdgeLengthP -= tsr_ushort((EdgeStopedByMaxLumaP | EdgeStopedByMinLumaP) >> SignShift);
		EdgeLengthN -= tsr_ushort((EdgeStopedByMaxLumaN | EdgeStopedByMinLumaN) >> SignShift);
	}
		
	bool bEdgeStopedByMinLumaP = (EdgeStopedByMinLumaP & SignMask) != 0;
	bool bEdgeStopedByMinLumaN = (EdgeStopedByMinLumaN & SignMask) != 0;
		
	bool bEdgeStopedByMaxLumaP = (EdgeStopedByMaxLumaP & SignMask) != 0;
	bool bEdgeStopedByMaxLumaN = (EdgeStopedByMaxLumaN & SignMask) != 0;
		
	bEdgeStopedByIncrementP = or(and(bEdgeMoreLuminous, bEdgeStopedByMaxLumaP), and(!bEdgeMoreLuminous, bEdgeStopedByMinLumaP));
	bEdgeStopedByIncrementN = or(and(bEdgeMoreLuminous, bEdgeStopedByMaxLumaN), and(!bEdgeMoreLuminous, bEdgeStopedByMinLumaN));
		
	bEdgeStopedByDecrementP = or(and(bEdgeMoreLuminous, bEdgeStopedByMinLumaP), and(!bEdgeMoreLuminous, bEdgeStopedByMaxLumaP));
	bEdgeStopedByDecrementN = or(and(bEdgeMoreLuminous, bEdgeStopedByMinLumaN), and(!bEdgeMoreLuminous, bEdgeStopedByMaxLumaN));


} // BrowseNeighborhoodBilinearRDNAOptimized

tsr_half ComputeDistanceToEdge(
	bool bEdgeStopedByIncrementN,
	bool bEdgeStopedByIncrementP,
	bool bEdgeStopedByDecrementN,
	bool bEdgeStopedByDecrementP,
	tsr_ushort EdgeLengthN,
	tsr_ushort EdgeLengthP)
{	
	tsr_half fEdgeLengthN = tsr_half(EdgeLengthN);
	tsr_half fEdgeLengthP = tsr_half(EdgeLengthP);
	tsr_half fEdgeLength = tsr_half(1.0) + fEdgeLengthN + fEdgeLengthP;
	tsr_half fEdgeInvLength = rcp(fEdgeLength);

	
	tsr_half TexelCenterOffsetToEdge = 0.0;

	const float Min = -0.5;

	FLATTEN
	if (!bEdgeStopedByIncrementN && !bEdgeStopedByIncrementP && !bEdgeStopedByDecrementP && !bEdgeStopedByDecrementN)
	{
		// No aliasing in the neighborhood.
		TexelCenterOffsetToEdge = tsr_half(0.0);
	}
	else if (bEdgeStopedByIncrementN && bEdgeStopedByIncrementP)
	{
		// Looks like InputC is concave detail
		TexelCenterOffsetToEdge = tsr_half(0.5);
	}
	else if (bEdgeStopedByDecrementN && bEdgeStopedByDecrementP)
	{
		// Looks like InputC is convex detail
		//TexelCenterOffsetToEdge = 0.5;
	}
	else if (bEdgeStopedByIncrementN) // && bEdgeStopedByDecrementP)
	{
		// Looks like staircasing from - to +
		TexelCenterOffsetToEdge = tsr_half(0.5) - (EdgeLengthN + tsr_half(0.5)) * fEdgeInvLength;
	}
	else if (bEdgeStopedByIncrementP) // && bEdgeStopedByDecrementN)
	{
		// Looks like staircasing from + to -
		TexelCenterOffsetToEdge = tsr_half(0.5) - (EdgeLengthP + tsr_half(0.5)) * fEdgeInvLength;
	}

	return TexelCenterOffsetToEdge;
} // ComputeDistanceToEdge()

tsr_half2 ComputeReprojectionBoundary(
	const uint BrowsingIterations,
	tsr_short2 EdgeSide,
	tsr_ushort EdgeLengthP,
	tsr_ushort EdgeLengthN,
	bool bEdgeStopedByIncrementP,
	bool bEdgeStopedByIncrementN,
	bool bEdgeStopedByDecrementP,
	bool bEdgeStopedByDecrementN)
{
	const tsr_half fMaxEdgeLength = tsr_half(BrowsingIterations + 1);
	const tsr_half fMaxEdgeInvLength = rcp(fMaxEdgeLength);

	tsr_half fEdgeLengthN = tsr_half(EdgeLengthN);
	tsr_half fEdgeLengthP = tsr_half(EdgeLengthP);
	tsr_half fEdgeLength = tsr_half(1.0) + fEdgeLengthN + fEdgeLengthP;
	tsr_half fEdgeInvLength = rcp(fEdgeLength);

	tsr_half MaxDistanceToEdge = tsr_half(0.0);
	tsr_half MaxEdgeInclination = tsr_half(0.0);
	{
		FLATTEN
		if (bEdgeStopedByDecrementP && bEdgeStopedByDecrementN)
		{
			MaxDistanceToEdge = (min(fEdgeLengthP, fEdgeLengthN) + tsr_half(0.5)) * fMaxEdgeInvLength;
			MaxEdgeInclination = select(fEdgeLengthP < fEdgeLengthN, -fMaxEdgeInvLength, tsr_half(0.0));
			MaxEdgeInclination = select(fEdgeLengthN < fEdgeLengthP, +fMaxEdgeInvLength, MaxEdgeInclination);
		}
		else if (bEdgeStopedByDecrementP)
		{
			MaxDistanceToEdge = (fEdgeLengthP + tsr_half(0.5)) * max(fEdgeInvLength, fMaxEdgeInvLength);
			MaxEdgeInclination = -max(fEdgeInvLength, fMaxEdgeInvLength);
		}
		else if (bEdgeStopedByDecrementN)
		{
			MaxDistanceToEdge = (fEdgeLengthN + tsr_half(0.5)) * max(fEdgeInvLength, fMaxEdgeInvLength);
			MaxEdgeInclination = +max(fEdgeInvLength, fMaxEdgeInvLength);
		}
		else
		{
			MaxDistanceToEdge = tsr_half(1.0);
			MaxEdgeInclination = tsr_half(0.0);
		}
	}

	tsr_half MinDistanceToEdge = tsr_half(0.0);
	tsr_half MinEdgeInclination = tsr_half(0.0);
	{
		FLATTEN
		if (bEdgeStopedByIncrementP && bEdgeStopedByIncrementN)
		{
			MinDistanceToEdge = 1.0 - (min(fEdgeLengthP, fEdgeLengthN) + tsr_half(0.5)) * fMaxEdgeInvLength;
			MinEdgeInclination = select(fEdgeLengthP < fEdgeLengthN, +fMaxEdgeInvLength, tsr_half(0.0));
			MinEdgeInclination = select(fEdgeLengthN < fEdgeLengthP, -fMaxEdgeInvLength, MinEdgeInclination);
		}
		else if (bEdgeStopedByIncrementP)
		{
			MinDistanceToEdge = 1.0 - (fEdgeLengthP + tsr_half(0.5)) * max(fMaxEdgeInvLength, fEdgeInvLength);
			MinEdgeInclination = +max(fEdgeInvLength, fMaxEdgeInvLength);
		}
		else if (bEdgeStopedByIncrementN)
		{
			MinDistanceToEdge = 1.0 - (fEdgeLengthN + tsr_half(0.5)) * max(fMaxEdgeInvLength, fEdgeInvLength);
			MinEdgeInclination = -max(fEdgeInvLength, fMaxEdgeInvLength);
		}
		else
		{
			MinDistanceToEdge = tsr_half(0.0);
			MinEdgeInclination = tsr_half(0.0);
		}
		
		MinDistanceToEdge = max(MinDistanceToEdge, rcp(tsr_half(1023.0)));
		//MinDistanceToEdge = max(MinDistanceToEdge, rcp(tsr_half(8.0)));
	}

	// Merge the distance to edge computation.
	tsr_half DistanceToEdge;
	tsr_half EdgeInclination;
	#if 1
	{
		//tsr_half DilateAmount = dot(tsr_half2(EdgeSide), tsr_half2(InputJitter));
		tsr_half DilateAmount = tsr_half(0.5);
		//DilateAmount = max(DilateAmount, rcp(tsr_half(8.0)));
		//DilateAmount += tsr_half(0.5);

		// Absolute mid dilate with spatial anti-aliasing.
		DistanceToEdge = max(MinDistanceToEdge, min(MaxDistanceToEdge, DilateAmount));
		EdgeInclination = tsr_half(0.0);
		EdgeInclination = select(MaxDistanceToEdge <= tsr_half(DilateAmount), MaxEdgeInclination, EdgeInclination);
		EdgeInclination = select(MinDistanceToEdge >  tsr_half(DilateAmount), MinEdgeInclination, EdgeInclination);
	}
	#elif 0
	{
		// Absolute minimum dilate with spatial anti-aliasing.
		DistanceToEdge = MinDistanceToEdge;
		EdgeInclination = MinEdgeInclination;
	}
	#elif 0
	{
		// Full dilate at rendering pixel granularity, but chamfer the extremity.
		DistanceToEdge = MaxDistanceToEdge;
		EdgeInclination = MaxEdgeInclination;
	}
	#else
	{
		// Full dilate at rendering pixel granularity.
		DistanceToEdge = tsr_half(1.0);
		EdgeInclination = tsr_half(0.0);
	}
	#endif

	#if 1
	{
		tsr_half2 fEdgeSide = tsr_half2(EdgeSide);
		tsr_half2 fDirectionP = tsr_half2(abs(fEdgeSide.y), abs(fEdgeSide.x));

		tsr_half2 EdgeDirection = fEdgeSide + fDirectionP * EdgeInclination;
		EdgeDirection *= rsqrt(dot(EdgeDirection, EdgeDirection));

		tsr_half2 K = saturate(tsr_half2(sign(EdgeDirection)));
		tsr_half2 E = tsr_half2(0.5, 0.5) + fEdgeSide * (0.5 - DistanceToEdge);

		tsr_half2 EK = K - E;
		tsr_half KFromEdge = dot(EK, EdgeDirection);
		tsr_half2 L = K - KFromEdge * EdgeDirection;

		tsr_half2 ReprojectionBoundary = K - L;
		return ReprojectionBoundary;
	}
	#else
	{
		tsr_half2 ReprojectionBoundary = tsr_half2(EdgeSide) * DistanceToEdge;
		return ReprojectionBoundary;
	}
	#endif
} // ComputeReprojectionBoundary()


//------------------------------------------------------- ENCODE ANTI-ALIASING OFFSET

// Spatial anti-aliasing encoding settings.
#define SPATIAL_ANTI_ALIASING_OFFSET_RANGE 0.5
#define SPATIAL_ANTI_ALIASING_OFFSET_BITDEPTH_PER_CHANNEL 4

CALL_SITE_DEBUGLOC
uint2 EncodeSpatialAntiAliasingOffset(tsr_half2x2 TexelOffset)
{
	const uint Mask = (1 << SPATIAL_ANTI_ALIASING_OFFSET_BITDEPTH_PER_CHANNEL) - 1;
	const uint Zero = Mask / 2;

	tsr_half Multiply = tsr_half(float(Zero) / SPATIAL_ANTI_ALIASING_OFFSET_RANGE);

	uint2 EncodedTexelOffset = 0;
	EncodedTexelOffset |= fastClamp(uint2(int2(round(TexelOffset[0] * Multiply)) + int(1 + Zero)), uint(1), Mask) << 0;
	EncodedTexelOffset |= fastClamp(uint2(int2(round(TexelOffset[1] * Multiply)) + int(1 + Zero)), uint(1), Mask) << SPATIAL_ANTI_ALIASING_OFFSET_BITDEPTH_PER_CHANNEL;
	return EncodedTexelOffset;
}

CALL_SITE_DEBUGLOC
uint EncodeSpatialAntiAliasingOffset(tsr_half2 TexelOffset)
{
	return dpv_lo(EncodeSpatialAntiAliasingOffset(dpv_interleave_mono_registers(TexelOffset)));
}

CALL_SITE_DEBUGLOC
tsr_half2x2 DecodeSpatialAntiAliasingOffset(tsr_ushort2 EncodedInputTexelOffset)
{
	const tsr_ushort Mask = tsr_ushort(1 << SPATIAL_ANTI_ALIASING_OFFSET_BITDEPTH_PER_CHANNEL) - tsr_ushort(1);
	const tsr_ushort Zero = Mask / tsr_ushort(2);
	
	tsr_half Multiply = tsr_half(SPATIAL_ANTI_ALIASING_OFFSET_RANGE / float(Zero));

	tsr_half2x2 TexelOffset;
	TexelOffset[0] = tsr_half2((EncodedInputTexelOffset >> tsr_ushort(0                                                )) & Mask) * Multiply - (tsr_half(1 + Zero) * Multiply);
	TexelOffset[1] = tsr_half2((EncodedInputTexelOffset >> tsr_ushort(SPATIAL_ANTI_ALIASING_OFFSET_BITDEPTH_PER_CHANNEL)) & Mask) * Multiply - (tsr_half(1 + Zero) * Multiply);

	return TexelOffset;
}

CALL_SITE_DEBUGLOC
tsr_half2 DecodeSpatialAntiAliasingOffset(tsr_ushort EncodedInputTexelOffset)
{
	return dpv_lo(DecodeSpatialAntiAliasingOffset(dpv_interleave_mono_registers(EncodedInputTexelOffset)));
}
