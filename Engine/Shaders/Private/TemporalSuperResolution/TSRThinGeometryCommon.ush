// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//------------------------------------------------------- INCLUDES
#include "TSRColorSpace.ush"
#include "TSRConvolutionNetworkPass.ush"

//------------------------------------------------------- Config

// Packs high contrast line and translucency bits into thin geometry coverage
// encoding.

// Coverage texture (PF_R8 UNORM)
//	ON: [0] high contrast line [1] translucency [2:7] coverage (6bit)
// OFF: [0] high contrast line [1:7] coverage (7bit)

#ifndef CONFIG_TRANSLUCENCY_COVERAGE_BIT
#define CONFIG_TRANSLUCENCY_COVERAGE_BIT 0
#endif

#ifndef CONFIG_INTENSITY_LINE_DETECTION
#define CONFIG_INTENSITY_LINE_DETECTION 0
#endif

#if CONFIG_TRANSLUCENCY_COVERAGE_BIT && CONFIG_INTENSITY_LINE_DETECTION
static const tsr_half kCoverageMaxValue			= tsr_half(63.0);
static const tsr_ushort kCoverageShift			= tsr_ushort(0x2);
static const tsr_ushort kTranslucencyBit		= tsr_ushort(0x2);
#elif CONFIG_INTENSITY_LINE_DETECTION
static const tsr_half kCoverageMaxValue			= tsr_half(127.0);
static const tsr_ushort kCoverageShift			= tsr_ushort(0x1);
static const tsr_ushort kTranslucencyBit		= tsr_ushort(0x0);
#elif CONFIG_TRANSLUCENCY_COVERAGE_BIT
static const tsr_half kCoverageMaxValue			= tsr_half(127.0);
static const tsr_ushort kCoverageShift			= tsr_ushort(0x1);
static const tsr_ushort kTranslucencyBit		= tsr_ushort(0x1);
#else
static const tsr_half kCoverageMaxValue			= tsr_half(255.0);
static const tsr_ushort kCoverageShift			= tsr_ushort(0x0);
#endif

static const tsr_ushort kHighContrastLineBit	= tsr_ushort(0x1);
//-------------------------------------------------------UTIL FUNCTION

float EncodeThinGeometryMaterial(uint ThinGeometryFlag)
{
	return ((float)ThinGeometryFlag) * rcp(2.0f);
}

CALL_SITE_DEBUGLOC
tsr_tensor_bool IsThinGeometry(tsr_tensor_half CurrentCoverage)
{
	return CurrentCoverage == tsr_tensor_half::Const(1.0);
}

CALL_SITE_DEBUGLOC
tsr_tensor_half ToBinaryCoverage(tsr_tensor_half EncodedThinGeometryMaterial)
{
	return floor(EncodedThinGeometryMaterial);
}

CALL_SITE_DEBUGLOC
void DecodeThinGeometry(
	tsr_tensor_ushort EncodedThinGeometryWeight,
	out tsr_tensor_half HistoryRelaxationWeight,
	out tsr_tensor_bool IsClusterHoleRegion)
{
	// use sign to encode?
	HistoryRelaxationWeight = tsr_tensor_half::CastFrom<tsr_ushort>(
		bit_shift_right(EncodedThinGeometryWeight, tsr_tensor_ushort::Const(tsr_ushort(0x1u)))) * rcp(tsr_half(127.0));
	IsClusterHoleRegion = tsr_tensor_bool::CastFrom<tsr_ushort>(bit_and(EncodedThinGeometryWeight, tsr_tensor_ushort::Const(tsr_ushort(0x1u))));
}

CALL_SITE_DEBUGLOC
tsr_tensor_ushort EncodeThinGeometry(
	tsr_tensor_half HistoryRelaxationWeight,
	tsr_tensor_bool IsClusterHoleRegion)
{
	tsr_tensor_ushort EncodedThinGeometry = bit_shift_left(
		tsr_tensor_ushort::CastFrom<tsr_half>(HistoryRelaxationWeight * tsr_half(127.0)), tsr_tensor_ushort::Const(tsr_ushort(0x1u)));
	EncodedThinGeometry = bit_or(EncodedThinGeometry,tsr_tensor_ushort::CastFrom<bool>(IsClusterHoleRegion));
	return EncodedThinGeometry;
}

//------------------------------------------------------- COVERAGE ENCODE/DECODE
// HadTranslucency bit resets to 0
tsr_half EncodeCoverage(tsr_half Coverage, bool bHighContrastLine, tsr_half Dither)
{
    tsr_ushort CoverageBits = tsr_ushort(clamp(saturate(Coverage) * kCoverageMaxValue + Dither, tsr_half(0.0), kCoverageMaxValue));
    tsr_ushort Packed = (CoverageBits << tsr_ushort(kCoverageShift))
						| select(bHighContrastLine, kHighContrastLineBit, tsr_ushort(0));
    return tsr_half(Packed) * rcp(tsr_half(255.0));
}

tsr_ushort GetTranslucencyOnlyBitMask(bool bIntensityLineDetectionEnabled)
{
#if CONFIG_INTENSITY_LINE_DETECTION
	return kTranslucencyBit;
#else
	return select(bIntensityLineDetectionEnabled, tsr_ushort(0x2), tsr_ushort(0x1));
#endif
}

tsr_half OverwriteCoverageWithTranslucencyBit(tsr_half EncodedCoverage, bool bHadTranslucency, bool bIntensityLineDetectionEnabled)
{
#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
	tsr_ushort CoverageBits = tsr_ushort(EncodedCoverage * tsr_half(255.0f) + tsr_half(0.5));
	tsr_ushort Packed = select(bHadTranslucency, GetTranslucencyOnlyBitMask(bIntensityLineDetectionEnabled), CoverageBits);
	return tsr_half(Packed) * rcp(tsr_half(255.0));
#else
	return EncodedCoverage;
#endif
}

CALL_SITE_DEBUGLOC
void DecodeCoverage(
    tsr_tensor_half RawValue,
    out tsr_tensor_half Coverage
#if CONFIG_INTENSITY_LINE_DETECTION
	,out tsr_tensor_half HighContrastLine
#endif
#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
    ,out tsr_tensor_half HadTranslucency
#endif
)
{
#if !CONFIG_INTENSITY_LINE_DETECTION && !CONFIG_TRANSLUCENCY_COVERAGE_BIT
	Coverage = RawValue;
#else
	tsr_tensor_ushort Packed = tsr_tensor_ushort::CastFrom<tsr_half>(RawValue * tsr_half(255.0) + tsr_half(0.5));
	Coverage = tsr_tensor_half::CastFrom<tsr_ushort>(bit_shift_right(Packed, tsr_tensor_ushort::Const(kCoverageShift))) * rcp(kCoverageMaxValue);

	#if CONFIG_INTENSITY_LINE_DETECTION
	HighContrastLine = tsr_tensor_half::CastFrom<tsr_ushort>(bit_and(Packed, tsr_tensor_ushort::Const(kHighContrastLineBit)));
	#endif

	#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
	HadTranslucency = saturate(tsr_tensor_half::CastFrom<tsr_ushort>(bit_and(Packed, tsr_tensor_ushort::Const(kTranslucencyBit))));
	#endif
#endif
}