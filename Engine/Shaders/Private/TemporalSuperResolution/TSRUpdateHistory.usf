// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRKernels.ush"
#include "TSRSpatialAntiAliasing.ush"
#include "TSRColorSpace.ush"
#include "TSRReprojectionField.ush"


//------------------------------------------------------- DEBUG

#define DEBUG_ARRAY_SIZE 8


//------------------------------------------------------- ENUMS

/** Shape of the scene color kernel. */
	// One single sample
	#define CONFIG_SAMPLES_1X1 0
	
	// Full 3x3
	#define CONFIG_SAMPLES_3X3 1
	
	// 3x3 plus
	#define CONFIG_SAMPLES_PLUS 2
	
	// 3x3 plus with a 6th sample in the closest corner.
	#define CONFIG_SAMPLES_PLUS_AND_CORNER 3

	// 3x3 plus but disable the furtherest samples for clamping box.
	#define CONFIG_SAMPLES_PLUS_DISABLE_FAR 4
	
	// 3x3 plus but move the furtherest towards the closest corner.
	#define CONFIG_SAMPLES_PLUS_MOVE_FAR 5


//------------------------------------------------------- CONFIG

#if DIM_UPDATE_QUALITY == 0 // Low
	#define CONFIG_SAMPLES CONFIG_SAMPLES_PLUS
	#define CONFIG_REJECTION_ANTI_ALIASING 0

#elif DIM_UPDATE_QUALITY == 1 // Medium
	#define CONFIG_SAMPLES CONFIG_SAMPLES_PLUS
	#define CONFIG_REJECTION_ANTI_ALIASING 1

#elif DIM_UPDATE_QUALITY == 2 // High
	#define CONFIG_SAMPLES CONFIG_SAMPLES_PLUS_MOVE_FAR
	#define CONFIG_REJECTION_ANTI_ALIASING 1

#elif DIM_UPDATE_QUALITY == 3 // Epic
	#define CONFIG_SAMPLES CONFIG_SAMPLES_PLUS_MOVE_FAR
	#define CONFIG_REJECTION_ANTI_ALIASING 1

#else
	#error Unknown history update quality
#endif


//------------------------------------------------------- DEFAULTS

/** Controls whether the shader is capable to do history resurrection. */
#define CONFIG_HISTORY_RESURRECTION 1

/** Controls whether the shader is capable to use the reprojection field. */
#define CONFIG_REPROJECTION_FIELD 1

/** Whether lens distortion is supported. */
#define CONFIG_LENS_DISTORTION TSR_SUPPORT_LENS_DISTORTION

/** Whether the history should do any rejection of any kind. */
#define CONFIG_CLAMP 1

/** Defines the number of samples of the scene color. */
#if CONFIG_SAMPLES == CONFIG_SAMPLES_1X1
	#define CONFIG_SAMPLES_COUNT 1
#elif CONFIG_SAMPLES == CONFIG_SAMPLES_3X3
	#define CONFIG_SAMPLES_COUNT 9
#elif CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_6
	#define CONFIG_SAMPLES_COUNT 6
#else
	#define CONFIG_SAMPLES_COUNT 5
#endif

/** Defines whether can completly skip the history reprojection when offscreen on fully rejected. */
#define CONFIG_SELECTIVE_REPROJECTION 1

/** Controls number of pixel to process per lane. */
#if CONFIG_COMPILE_FP16
	// Take advantage of RDNA's v_pk_*_{uif}16 instructions
	#define CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION 1
#else
	#define CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION 0
#endif

#ifndef PLATFORM_SPECIFIC_ISOLATE
	#define PLATFORM_SPECIFIC_ISOLATE ISOLATE
#endif


//------------------------------------------------------- CONSTANTS

#define TILE_SIZE 8

#define CONFIG_METADATA_CHANNELS 1

#if CONFIG_METADATA_CHANNELS == 2
	#define tsr_halfM tsr_half2
	#define tsr_halfMx2 tsr_half2x2
#elif CONFIG_METADATA_CHANNELS == 1
	#define tsr_halfM tsr_half
	#define tsr_halfMx2 tsr_half2
#else
	#error Unknown CONFIG_METADATA_CHANNELS
#endif


//------------------------------------------------------- PARAMETERS

Texture2D<tsr_halfC> InputSceneColorTexture;

Texture2D<tsr_half4> HistoryRejectionTexture;

#if CONFIG_REPROJECTION_FIELD
	Texture2D<uint> ReprojectionBoundaryTexture;
	Texture2D<uint> ReprojectionJacobianTexture;
#endif
Texture2D<uint> ReprojectionVectorTexture;
Texture2D<tsr_ushort2> AntiAliasingTexture;

#if CONFIG_LENS_DISTORTION
	Texture2D<float2> PrevDistortingDisplacementTexture;
	Texture2D<float2> ResurrectedDistortingDisplacementTexture;
	Texture2D<float2> UndistortingDisplacementTexture;
	float DistortionOverscan;
#endif

FScreenTransform HistoryPixelPosToViewportUV;
FScreenTransform ViewportUVToInputPPCo;
FScreenTransform HistoryPixelPosToScreenPos;
FScreenTransform HistoryPixelPosToInputPPCo;
float3 HistoryQuantizationError;
float HistorySampleCount;
float HistoryHisteresis;
float WeightClampingRejection;
float WeightClampingPixelSpeedAmplitude;
float InvWeightClampingPixelSpeed;
float InputToHistoryFactor;
float ResurrectionFrameIndex;
float PrevFrameIndex;
#if CONFIG_LENS_DISTORTION
	uint bLensDistortion;
#endif
#if CONFIG_REPROJECTION_FIELD
	uint bReprojectionField;
#endif
uint bGenerateOutputMip1;
uint bGenerateOutputMip2;
uint bGenerateOutputMip3;
uint HistoryArrayIndices_HighFrequency;

Texture2DArray<tsr_halfC> PrevHistoryColorTexture;
Texture2DArray<tsr_halfM> PrevHistoryMetadataTexture;

RWTexture2DArray<tsr_halfC> HistoryColorOutput;
RWTexture2DArray<tsr_halfM> HistoryMetadataOutput;

RWTexture2DArray<tsr_halfC> SceneColorOutputMip1;


//------------------------------------------------------- LDS

#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
#define GROUP_SHARED_SIZE (TILE_SIZE * TILE_SIZE / 2)
#else
#define GROUP_SHARED_SIZE (TILE_SIZE * TILE_SIZE)
#endif

groupshared tsr_half4 SharedArray0[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray1[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray2[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray3[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray4[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray5[GROUP_SHARED_SIZE];

groupshared tsr_half4 SharedArray6[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray7[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray8[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray9[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray10[GROUP_SHARED_SIZE];
groupshared tsr_half4 SharedArray11[GROUP_SHARED_SIZE];


//------------------------------------------------------- WAVE

#if COMPILER_SUPPORTS_MED3

CALL_SITE_DEBUGLOC
tsr_half3x2 med3(tsr_half3x2 a, tsr_half3x2 b, tsr_half3x2 c)
{
	return tsr_half3x2(
		med3(a[0], b[0], c[0]),
		med3(a[1], b[1], c[1]),
		med3(a[2], b[2], c[2]));
} 

CALL_SITE_DEBUGLOC
tsr_half4x2 med3(tsr_half4x2 a, tsr_half4x2 b, tsr_half4x2 c)
{
	return tsr_half4x2(
		med3(a[0], b[0], c[0]),
		med3(a[1], b[1], c[1]),
		med3(a[2], b[2], c[2]),
		med3(a[3], b[3], c[3]));
} 

#endif

#if PLATFORM_SUPPORTS_WAVE_BROADCAST

CALL_SITE_DEBUGLOC
tsr_half3x2 WaveBroadcast(const FWaveBroadcastSettings Settings, tsr_half3x2 v)
{
	return tsr_half3x2(
		WaveBroadcast(Settings, v[0]),
		WaveBroadcast(Settings, v[1]),
		WaveBroadcast(Settings, v[2]));
} 

CALL_SITE_DEBUGLOC
tsr_half4x2 WaveBroadcast(const FWaveBroadcastSettings Settings, tsr_half4x2 v)
{
	return tsr_half4x2(
		WaveBroadcast(Settings, v[0]),
		WaveBroadcast(Settings, v[1]),
		WaveBroadcast(Settings, v[2]),
		WaveBroadcast(Settings, v[3]));
} 

#endif


//------------------------------------------------------- FUNCTIONS

CALL_SITE_DEBUGLOC 
tsr_ushort2x2 Map8x8Tile2x2LaneDPV(uint GroupThreadIndex)
{
	tsr_ushort2 GroupId = Map8x8Tile2x2Lane(GroupThreadIndex);
#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
	return dpv_interleave_registers(GroupId, GroupId + tsr_ushort2(0,  8 / 2));
#else
	return dpv_interleave_mono_registers(GroupId);
#endif
}

CALL_SITE_DEBUGLOC 
tsr_half3x2 GetColorChannel(tsr_half4x2 Color)
{
	return dpv_interleave_registers(
		dpv_lo(Color).rgb,
		dpv_hi(Color).rgb);
}

void ComputeInputKernelSamplePosition(
	tsr_short2x2 PixelPos,
	tsr_half2x2 dKO,
	const uint SampleId,
	int2 PixelPosMin,
	int2 PixelPosMax,
	out tsr_short2x2 SamplePixelPos,
	out tsr_half2x2 PixelOffset,
	out bool2 bIsDuplicatedSample,
	out bool2 bIsDisabledSample)
{
	tsr_short2x2 iKOSign;
	tsr_half2x2 KOSign;
	#if CONFIG_COMPILE_FP16
		iKOSign[0] = int16_t(1) - int16_t2((asuint16(dKO[0]) & uint16_t(0x8000)) >> uint16_t(14));
		iKOSign[1] = int16_t(1) - int16_t2((asuint16(dKO[1]) & uint16_t(0x8000)) >> uint16_t(14));
		KOSign[0] = asfloat16(asuint16(half(1.0)).xx | (asuint16(dKO[0]) & uint16_t(0x8000)));
		KOSign[1] = asfloat16(asuint16(half(1.0)).xx | (asuint16(dKO[1]) & uint16_t(0x8000)));
	#else
		iKOSign = dpv_interleave_registers(
			SignFastInt(dpv_lo(dKO)), SignFastInt(dpv_hi(dKO)));
		KOSign[0] = asfloat(asuint(1.0).xx | (asuint(dKO[0]) & uint(0x80000000)));
		KOSign[1] = asfloat(asuint(1.0).xx | (asuint(dKO[1]) & uint(0x80000000)));
	#endif
	
	bIsDuplicatedSample = false;
	bIsDisabledSample = false;

	#if CONFIG_SAMPLES == CONFIG_SAMPLES_3X3
	{
		tsr_short2 iPixelOffset = tsr_short2(kOffsets3x3[kSquareIndexes3x3[SampleId]]);
		PixelOffset = dpv_interleave_registers(tsr_half2(iPixelOffset), tsr_half2(iPixelOffset));
				
		SamplePixelPos = PixelPos + ClampPixelOffset(
			PixelPos,
			dpv_interleave_registers(iPixelOffset, iPixelOffset), iPixelOffset,
			PixelPosMin, PixelPosMax);
	}
	#elif CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_MOVE_FAR
	{
		tsr_half2x2 SquareShape;
		#if CONFIG_COMPILE_FP16
			SquareShape[0] = asfloat16(asuint16(half(1.0)).xx | (asuint16(abs(dKO[0]) - tsr_half(0.25).xx) & uint16_t(0x8000)));
			SquareShape[1] = asfloat16(asuint16(half(1.0)).xx | (asuint16(abs(dKO[1]) - tsr_half(0.25).xx) & uint16_t(0x8000)));
		#else
			SquareShape[0] = asfloat(asuint(1.0).xx | (asuint(abs(dKO[0]) - float(0.25).xx) & 0x8000u));
			SquareShape[1] = asfloat(asuint(1.0).xx | (asuint(abs(dKO[1]) - float(0.25).xx) & 0x8000u));
		#endif

		PixelOffset = tsr_half(0);
		SamplePixelPos = PixelPos;

		if (SampleId == 0)
		{
			// NOP
		}
		else if (SampleId == 1)
		{
			PixelOffset[0] = KOSign[0];
			
			SamplePixelPos[0] += tsr_short2(PixelOffset[0]);
			SamplePixelPos[0] = ClampPixelOffset(SamplePixelPos, PixelPosMin, PixelPosMax)[0];
		}
		else if (SampleId == 2)
		{
			PixelOffset[1] = KOSign[1];
			
			SamplePixelPos[1] += tsr_short2(PixelOffset[1]);
			SamplePixelPos[1] = ClampPixelOffset(SamplePixelPos, PixelPosMin, PixelPosMax)[1];
		}
		else if (SampleId == 3)
		{
			PixelOffset[0] = SquareShape[0] * KOSign[0];
			PixelOffset[1] = saturate(SquareShape[0]) * KOSign[1];
			
			SamplePixelPos[0] += tsr_short2(PixelOffset[0]);
			SamplePixelPos[1] += tsr_short2(PixelOffset[1]);
			SamplePixelPos = ClampPixelOffset(SamplePixelPos, PixelPosMin, PixelPosMax);
		}
		else // if (SampleId == 4)
		{
			PixelOffset[0] = saturate(SquareShape[1]) * KOSign[0];
			PixelOffset[1] = SquareShape[1] * KOSign[1];

			bIsDuplicatedSample = (SquareShape[0] + SquareShape[1]) == tsr_half(2.0);
			
			SamplePixelPos[0] += tsr_short2(PixelOffset[0]);
			SamplePixelPos[1] += tsr_short2(PixelOffset[1]);
			SamplePixelPos = ClampPixelOffset(SamplePixelPos, PixelPosMin, PixelPosMax);
		}
	}
	#elif CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS || CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_DISABLE_FAR || CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_AND_CORNER
	{
		#if CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_AND_CORNER
		if (SampleId == 5)
		{
			PixelOffset = KOSign;
			SamplePixelPos = ClampPixelOffset(PixelPos + iKOSign, PixelPosMin, PixelPosMax);
		}
		else
		#endif
		{
			const tsr_short2 iPixelOffset = tsr_short2(kOffsets3x3[kPlusIndexes3x3[SampleId]]);
			PixelOffset = dpv_interleave_registers(tsr_half2(iPixelOffset), tsr_half2(iPixelOffset));
					
			SamplePixelPos = PixelPos + ClampPixelOffset(
				PixelPos,
				dpv_interleave_registers(iPixelOffset, iPixelOffset), iPixelOffset,
				PixelPosMin, PixelPosMax);
				
			#if CONFIG_SAMPLES == CONFIG_SAMPLES_PLUS_DISABLE_FAR
			if (all(iPixelOffset == tsr_short2(+1, +0)))
			{
				bIsDisabledSample = dKO[0] < tsr_half(-0.25);
			}
			else if (all(iPixelOffset == tsr_short2(+0, +1)))
			{
				bIsDisabledSample = dKO[1] < tsr_half(-0.25);
			}
			else if (all(iPixelOffset == tsr_short2(-1, +0)))
			{
				bIsDisabledSample = dKO[0] > tsr_half(+0.25);
			}
			else if (all(iPixelOffset == tsr_short2(+0, -1)))
			{
				bIsDisabledSample = dKO[1] > tsr_half(+0.25);
			}
			#endif
		}
	}
	#elif CONFIG_SAMPLES == CONFIG_SAMPLES_1X1
	{
		SamplePixelPos = PixelPos;
		PixelOffset = 0;
	}
	#else
		#error Unknown sample count
	#endif
}

template<typename T>
T BilinearSampleColorHistory(Texture2DArray<T> Texture, float3 SampleUV)
#if 1
{
	return Texture.SampleLevel(GlobalBilinearClampedSampler, SampleUV, 0);
}
#else
{
	FBilinearSampleInfos BilinearInter = GetBilinearSampleLevelInfos(SampleUV.xy, PrevHistoryInfo_Extent, PrevHistoryInfo_ExtentInverse);

	T Return = 0.0;

	UNROLL_N(4)
	for (uint i = 0; i < 4; i++)
	{
		float BilinearWeight = GetSampleWeight(BilinearInter, i);
		uint2 PixelPos = GetSamplePixelCoord(BilinearInter, i);
		PixelPos = fastClamp(PixelPos, PrevHistoryInfo_ViewportMin, PrevHistoryInfo_ViewportMax - 1);

		T RawSample = Texture[uint3(PixelPos, SampleUV.z)];
		
		if (View.GeneralPurposeTweak == 1.0)
			RawSample.rgb *= tsr_half(HdrWeight4(RawSample.rgb));

		Return += RawSample * tsr_half(BilinearWeight);
	}
	
	if (View.GeneralPurposeTweak == 1.0)
		Return.rgb *= tsr_half(HdrWeightInvY(Luma4(Return.rgb)));

	return Return;
}
#endif

tsr_halfCx2 DownsampleSceneColor(tsr_halfCx2 SceneColor, uint LocalGroupThreadIndex, const uint2 XorButterFly)
{
	SceneColor = SceneColor * tsr_half(0.25);
	// Forces the * tsr_half(0.25) to be applied before to avoid turning bright pixels to +inf in the adds below.
	#if CONFIG_FP16_PRECISE_MULTIPLY_ORDER
		#if CONFIG_SCENE_COLOR_ALPHA
			const tsr_halfC ColorMax10BitsFloat = tsr_half(Max10BitsFloat * 0.25).xxxx;
		#else
			const tsr_halfC ColorMax10BitsFloat = tsr_half(Max10BitsFloat * 0.25).xxx;
		#endif
		SceneColor = dpv_min(SceneColor, ColorMax10BitsFloat);
	#endif

	#if PLATFORM_SUPPORTS_WAVE_BROADCAST
	if (uint(XorButterFly.x * 2) <= WaveGetLaneCount())
	{
		FWaveBroadcastSettings Horizontal = InitWaveXorButterfly(XorButterFly.x);
		SceneColor += WaveBroadcast(Horizontal, SceneColor);
	}
	else
	#endif
	{
		#if CONFIG_SCENE_COLOR_ALPHA
			SharedArray0[LocalGroupThreadIndex] = dpv_lo(SceneColor);
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				SharedArray1[LocalGroupThreadIndex] = dpv_hi(SceneColor);
			#endif
		#else
			SharedArray0[LocalGroupThreadIndex] = tsr_half4(dpv_lo(SceneColor), 0.0);
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				SharedArray1[LocalGroupThreadIndex] = tsr_half4(dpv_hi(SceneColor), 0.0);
			#endif
		#endif
		
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST
			const bool bBarrier = XorButterFly.x * 0x2 > WaveGetLaneCount();
		#else
			const bool bBarrier = XorButterFly.x * 0x2 > 0x8;
		#endif

		if (bBarrier)
		{
			GroupMemoryBarrierWithGroupSync();
		}
		
		uint OtherGroupThreadIndex = LocalGroupThreadIndex ^ (XorButterFly.x % GROUP_SHARED_SIZE);
		tsr_halfCx2 OtherSceneColor;
		#if CONFIG_SCENE_COLOR_ALPHA
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				OtherSceneColor = dpv_interleave_registers(SharedArray0[OtherGroupThreadIndex], SharedArray1[OtherGroupThreadIndex]);
			#else
				OtherSceneColor = dpv_interleave_mono_registers(SharedArray0[OtherGroupThreadIndex]);
			#endif
		#else
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				OtherSceneColor = dpv_interleave_registers(SharedArray0[OtherGroupThreadIndex].rgb, SharedArray1[OtherGroupThreadIndex].rgb);
			#else
				OtherSceneColor = dpv_interleave_mono_registers(SharedArray0[OtherGroupThreadIndex].rgb);
			#endif
		#endif
		
		if (bBarrier)
		{
			GroupMemoryBarrierWithGroupSync();
		}
		
		SceneColor += OtherSceneColor;
	}
	
	if (XorButterFly.y == 0x20 && CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION)
	{
		SceneColor = dpv_interleave_mono_registers(dpv_lo(SceneColor) + dpv_hi(SceneColor));
	}
	#if PLATFORM_SUPPORTS_WAVE_BROADCAST
	else if (uint(XorButterFly.y * 2) <= WaveGetLaneCount())
	{
		FWaveBroadcastSettings Vertical = InitWaveXorButterfly(XorButterFly.y);
		SceneColor += WaveBroadcast(Vertical, SceneColor);
	}
	else
	#endif
	{
		#if CONFIG_SCENE_COLOR_ALPHA
			SharedArray0[LocalGroupThreadIndex] = dpv_lo(SceneColor);
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				SharedArray1[LocalGroupThreadIndex] = dpv_hi(SceneColor);
			#endif
		#else
			SharedArray0[LocalGroupThreadIndex] = tsr_half4(dpv_lo(SceneColor), 0.0);
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				SharedArray1[LocalGroupThreadIndex] = tsr_half4(dpv_hi(SceneColor), 0.0);
			#endif
		#endif
		
		#if PLATFORM_SUPPORTS_WAVE_BROADCAST
			const bool bBarrier = XorButterFly.y * 0x2 > WaveGetLaneCount();
		#else
			const bool bBarrier = XorButterFly.y * 0x2 > 0x8;
		#endif
		
		if (bBarrier)
		{
			GroupMemoryBarrierWithGroupSync();
		}
		
		uint OtherGroupThreadIndex = LocalGroupThreadIndex ^ (XorButterFly.y % GROUP_SHARED_SIZE);
		tsr_halfCx2 OtherSceneColor;
		#if CONFIG_SCENE_COLOR_ALPHA
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				OtherSceneColor = dpv_interleave_registers(SharedArray0[OtherGroupThreadIndex], SharedArray1[OtherGroupThreadIndex]);
			#else
				OtherSceneColor = dpv_interleave_mono_registers(SharedArray0[OtherGroupThreadIndex]);
			#endif
		#else
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				OtherSceneColor = dpv_interleave_registers(SharedArray0[OtherGroupThreadIndex].rgb, SharedArray1[OtherGroupThreadIndex].rgb);
			#else
				OtherSceneColor = dpv_interleave_mono_registers(SharedArray0[OtherGroupThreadIndex].rgb);
			#endif
		#endif
		
		if (bBarrier)
		{
			GroupMemoryBarrierWithGroupSync();
		}
		
		SceneColor += OtherSceneColor;
	}
	return SceneColor;
}

#if CONFIG_LENS_DISTORTION
/** Samples a displacement texture and crops any values beyond the texture's original bounds */
float2 SampleDisplacementTexture(Texture2D<float2> DisplacementTexture, float2 TexCoord, float2 ViewportUV)
{
	// Convert the destination viewport UV to the displacement map's original frustum, and compute whether this value is in or out of that frustum
	float2 UnoverscannedUV = (ViewportUV - float2(0.5, 0.5)) * DistortionOverscan + float2(0.5, 0.5);
	bool bCropPixel = UnoverscannedUV.x < 0.0 || UnoverscannedUV.x > 1.0 || UnoverscannedUV.y < 0.0 || UnoverscannedUV.y > 1.0;

	return select(bCropPixel, float2(0.0, 0.0), DisplacementTexture.SampleLevel(GlobalBilinearClampedSampler, TexCoord, 0));
}

float2 ApplyDisplacementTextureOnScreenPos(Texture2D<float2> DisplacementTexture, float2 DistortedScreenPos, float2 ViewportScreenPos)
{
	float2 DistortedUV = ScreenPosToViewportUV(DistortedScreenPos);
	float2 ViewportUV = ScreenPosToViewportUV(ViewportScreenPos);
	
	float2 UndistortedUV = DistortedUV + SampleDisplacementTexture(DisplacementTexture, DistortedUV, ViewportUV);
			
	return ViewportUVToScreenPos(UndistortedUV);
}
#endif

#if CONFIG_REPROJECTION_FIELD

void FetchVelocityDilateBoundary(
	tsr_short2x2 InputPixelPos,
	out uint EncodedReprojectionBoundary[DPV_PIXEL_PER_LANE],
	out bool bApplyReprojectionBoundary)
{
	BRANCH
	if (bReprojectionField)
	PLATFORM_SPECIFIC_ISOLATE
	{
		UNROLL_N(DPV_PIXEL_PER_LANE)
		for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
		{
			tsr_ushort2 LocalInputPixelPos = dpv_access_pixel(InputPixelPos, PixelId);
			
			EncodedReprojectionBoundary[PixelId] = ReprojectionBoundaryTexture[LocalInputPixelPos];
		}
		
		const uint EncodedReprojectionNoBoundary = EncodeReprojectionBoundary(/* ReprojectionOffset = */ tsr_short2(0, 0), /* ReprojectionBoundary = */ kFullDilateBoundary);
		bApplyReprojectionBoundary = false;
		UNROLL_N(DPV_PIXEL_PER_LANE)
		for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
		{
			bApplyReprojectionBoundary |= EncodedReprojectionBoundary[PixelId] != EncodedReprojectionNoBoundary;
		}
	}
	else
	{
		UNROLL_N(DPV_PIXEL_PER_LANE)
		for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
		{
			EncodedReprojectionBoundary[PixelId] = 0;
		}
		bApplyReprojectionBoundary = false;
	}
} // FetchVelocityDilateBoundary

#endif // CONFIG_REPROJECTION_FIELD


//------------------------------------------------------- ENTRY POINT

#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
	[numthreads(TILE_SIZE * TILE_SIZE / 2, 1, 1)]
#else
	[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
#endif
void UpdateHistoryCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
		uint GroupWaveIndex = GetGroupWaveIndex(GroupThreadIndex, TILE_SIZE * TILE_SIZE / 2);
	#else
		uint GroupWaveIndex = GetGroupWaveIndex(GroupThreadIndex, TILE_SIZE * TILE_SIZE);
	#endif
	
	#if DEBUG_OUTPUT
	float4x2 Debug[DEBUG_ARRAY_SIZE];
	for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
	{
		Debug[DebugId] = 0.0;
	}
	#endif

	tsr_short2x2 HistoryPixelPos = dpv_add(
		tsr_short2(GroupId) * tsr_short2(TILE_SIZE, TILE_SIZE),
		tsr_short2x2(Map8x8Tile2x2LaneDPV(GroupThreadIndex)));
		
	float2x2 ScreenPos;
	float2x2 InputPPCo;
	#if CONFIG_LENS_DISTORTION
	BRANCH
	if (bLensDistortion)
	{
		float2x2 DestViewportUV = ApplyScreenTransform(float2x2(HistoryPixelPos), HistoryPixelPosToViewportUV);
		float2x2 SrcViewportUV;
		UNROLL_N(DPV_PIXEL_PER_LANE)
		for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
		{
			float2 LocalDestViewportUV = dpv_access_pixel(DestViewportUV, PixelId);			
			float2 LocalSrcViewportUV = LocalDestViewportUV + SampleDisplacementTexture(UndistortingDisplacementTexture, LocalDestViewportUV, LocalDestViewportUV);
			
			SrcViewportUV[0][PixelId] = LocalSrcViewportUV.x;
			SrcViewportUV[1][PixelId] = LocalSrcViewportUV.y;
		}

		ScreenPos[0][0] = ViewportUVToScreenPos(dpv_access_pixel(SrcViewportUV, 0)).x;
		ScreenPos[1][0] = ViewportUVToScreenPos(dpv_access_pixel(SrcViewportUV, 0)).y;
		ScreenPos[0][1] = ViewportUVToScreenPos(dpv_access_pixel(SrcViewportUV, 1)).x;
		ScreenPos[1][1] = ViewportUVToScreenPos(dpv_access_pixel(SrcViewportUV, 1)).y;
		
		// Pixel coordinate of the center of output pixel O in the input viewport.
		InputPPCo = ApplyScreenTransform(SrcViewportUV, ViewportUVToInputPPCo);
	}
	else
	#endif
	{
		ScreenPos = ApplyScreenTransform(float2x2(HistoryPixelPos), HistoryPixelPosToScreenPos);
	
	// Pixel coordinate of the center of output pixel O in the input viewport.
		InputPPCo = ApplyScreenTransform(float2x2(HistoryPixelPos), HistoryPixelPosToInputPPCo);
	}

	// Pixel coordinate of the center of the nearest input pixel K in the input viewport.
	float2x2 InputPPCk = floor(InputPPCo) + 0.5;
	
	tsr_short2x2 InputPixelPos = ClampPixelOffset(
		tsr_short2x2(InputPPCo),
		InputPixelPosMin, InputPixelPosMax);
		
	// Fetch the reprojection anti-aliasing.
	#if CONFIG_REPROJECTION_FIELD
	uint EncodedReprojectionBoundary[DPV_PIXEL_PER_LANE];
	bool bApplyReprojectionBoundary;
	FetchVelocityDilateBoundary(InputPixelPos, /* out */ EncodedReprojectionBoundary, /* out */ bApplyReprojectionBoundary);
	#endif // CONFIG_REPROJECTION_FIELD

	// Fetch reprojection-related information.
	float2x2 PrevScreenPos = ScreenPos;
	tsr_half2 LowFrequencyRejection      = tsr_half(1.0).xx;
	tsr_half2 DisableHistoryClamp        = tsr_half(1.0).xx;
	tsr_half2 DecreaseValidityMultiplier = tsr_half(0.0).xx;
	tsr_half2 OutputPixelVelocity = tsr_half(0.0).xx;
	tsr_half2 NoiseFiltering = tsr_half(0.0).xx;
	tsr_half2 ReprojectionUpscaleCorrection = tsr_half(1.0).xx;
	bool2 bIsParallaxRejected = false;
	#if CONFIG_HISTORY_RESURRECTION
		bool2 bIsHistoryResurrection = false;
	#else
		const bool2 bIsHistoryResurrection = false;
	#endif

	#if CONFIG_REJECTION_ANTI_ALIASING
		tsr_half2x2 SpatialAntiAliasingOffset = dpv_interleave_mono_registers(tsr_half(0.0).xx);
	#endif

	PLATFORM_SPECIFIC_ISOLATE
	{
		uint RawEncodedReprojectionVector[DPV_PIXEL_PER_LANE];
		tsr_half4 RawHistoryRejection[DPV_PIXEL_PER_LANE];
		
		#if CONFIG_REJECTION_ANTI_ALIASING
			tsr_ushort2 RawEncodedInputTexelOffset[DPV_PIXEL_PER_LANE];
		#endif

		#if CONFIG_REPROJECTION_FIELD
			uint RawEncodedReprojectionJacobian[DPV_PIXEL_PER_LANE];
			tsr_half2 JacobianCoordinate[DPV_PIXEL_PER_LANE];
			tsr_half4 RawDilatedHistoryRejection[DPV_PIXEL_PER_LANE];
		#endif

		// Issue overlapped texture fetches
		PLATFORM_SPECIFIC_ISOLATE
		{
			tsr_half2x2 dInputKO = tsr_half2x2(InputPPCo - InputPPCk);

			UNROLL_N(DPV_PIXEL_PER_LANE)
			for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
			{
				tsr_ushort2 LocalInputPixelPos = dpv_access_pixel(InputPixelPos, PixelId);

				#if CONFIG_REPROJECTION_FIELD
				tsr_short2 BoundaryDilateOffset;
				BRANCH
				if (bApplyReprojectionBoundary)
				{
					tsr_short2 ReprojectionOffset  = DecodeReprojectionOffset(EncodedReprojectionBoundary[PixelId]);
					tsr_half2 ReprojectionBoundary = DecodeReprojectionBoundary(EncodedReprojectionBoundary[PixelId]);

					bool bHistoryPixelWithinOffsetBoundary = IsHistoryPixelWithinOffsetBoundary(
						dpv_access_pixel(dInputKO, PixelId),
						ReprojectionBoundary,
						1.0 / InputToHistoryFactor);
					BoundaryDilateOffset = select(bHistoryPixelWithinOffsetBoundary, ReprojectionOffset, -ReprojectionOffset);
					
					//Debug[0][0][PixelId] = bHistoryPixelWithinOffsetBoundary && any(ReprojectionOffset != tsr_short(0));
					//Debug[0][1][PixelId] = ((LocalInputPixelPos.x ^ LocalInputPixelPos.y) & 0x1) == 0;
				}
				else
				{
					BoundaryDilateOffset = tsr_short2(0, 0);
				}
				
				JacobianCoordinate[PixelId] = dpv_access_pixel(dInputKO, PixelId) - tsr_half2(BoundaryDilateOffset);
				tsr_ushort2 LocalInputPixelPosWithReprojectionAA = LocalInputPixelPos + BoundaryDilateOffset;
				#else
				tsr_ushort2 LocalInputPixelPosWithReprojectionAA = LocalInputPixelPos;
				#endif
				
				RawEncodedReprojectionVector[PixelId]   = ReprojectionVectorTexture[LocalInputPixelPosWithReprojectionAA];
				#if CONFIG_REPROJECTION_FIELD
					RawEncodedReprojectionJacobian[PixelId] = ReprojectionJacobianTexture[LocalInputPixelPosWithReprojectionAA];
					RawDilatedHistoryRejection[PixelId] = HistoryRejectionTexture[LocalInputPixelPosWithReprojectionAA];
				#endif

				RawHistoryRejection[PixelId] = HistoryRejectionTexture[LocalInputPixelPos];
				
				#if CONFIG_REJECTION_ANTI_ALIASING
				{
					RawEncodedInputTexelOffset[PixelId] = AntiAliasingTexture[LocalInputPixelPos];
				}
				#endif
			}
		}
		
		// Process texture fetches.
		PLATFORM_SPECIFIC_ISOLATE
		{
			// Process velocity
			{
				float2x2 ReprojectionVector;

				for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
				{
					#if CONFIG_REPROJECTION_FIELD
					float2 ReprojectionScreenPosCorrection;
					BRANCH
					if (bReprojectionField)
					{
						uint EncodedReprojectionJacobian = RawEncodedReprojectionJacobian[PixelId];
						tsr_half2x2 ReprojectionJacobian = DecodeReprojectionJacobian(EncodedReprojectionJacobian);
						
						float2 ReprojectionPixelPosCorrection = float2(mul(JacobianCoordinate[PixelId], ReprojectionJacobian));
						//ReprojectionScreenPosCorrection = ReprojectionPixelPosCorrection * (float2(2, -2) * InputInfo_ViewportSizeInverse);
						ReprojectionScreenPosCorrection = ReprojectionPixelPosCorrection * InputPixelPosToScreenPos.xy;

						ReprojectionUpscaleCorrection[PixelId] = rcp(max(ComputeReprojectionUpscaleFactorFromJacobian(ReprojectionJacobian), 1.0));
					}
					else
					{
						ReprojectionUpscaleCorrection[PixelId] = tsr_half(1.0);
						ReprojectionScreenPosCorrection = float(0.0);
					}
					#else // !CONFIG_REPROJECTION_FIELD
					const float2 ReprojectionScreenPosCorrection = 0.0;
					#endif
					
					uint EncodedReprojectionVector = RawEncodedReprojectionVector[PixelId];
					float2 LocalReprojectionVector = DecodeReprojectionVector(EncodedReprojectionVector) + float2(ReprojectionScreenPosCorrection);

					ReprojectionVector[0][PixelId] = LocalReprojectionVector[0];
					ReprojectionVector[1][PixelId] = LocalReprojectionVector[1];
				}

				PrevScreenPos = ScreenPos - ReprojectionVector;
				OutputPixelVelocity = tsr_half2(dpv_length(dpv_mul(ReprojectionVector, HistoryInfo_ViewportSize)));
			}
			
			// Unpacks bits masks
			{
				#if CONFIG_REPROJECTION_FIELD
					tsr_ushort2 BitMask = tsr_ushort2(round(dpv_interleave_registers_array(RawDilatedHistoryRejection)[3] * tsr_half(255.0)));
				#else
					tsr_ushort2 BitMask = tsr_ushort2(round(dpv_interleave_registers_array(RawHistoryRejection)[3] * tsr_half(255.0)));
				#endif
				bIsParallaxRejected = (BitMask & tsr_ushort(0x1)) == tsr_ushort(0);
				#if CONFIG_HISTORY_RESURRECTION
					bIsHistoryResurrection = (BitMask & tsr_ushort(0x2)) != tsr_ushort(0);
				#endif
			}

			// Process input texel, forcing to pack register.
			LowFrequencyRejection      = dpv_force_interleave_registers_array(RawHistoryRejection)[0];
			DisableHistoryClamp        = dpv_force_interleave_registers_array(RawHistoryRejection)[1];
			DecreaseValidityMultiplier = dpv_force_interleave_registers_array(RawHistoryRejection)[2];
			
			#if CONFIG_REPROJECTION_FIELD
			{
				LowFrequencyRejection      = min(LowFrequencyRejection     , dpv_force_interleave_registers_array(RawDilatedHistoryRejection)[0]);
				DisableHistoryClamp        = min(DisableHistoryClamp       , dpv_force_interleave_registers_array(RawDilatedHistoryRejection)[1]);
				DecreaseValidityMultiplier = max(DecreaseValidityMultiplier, dpv_force_interleave_registers_array(RawDilatedHistoryRejection)[2]);
			}
			#endif
		}

		#if CONFIG_REJECTION_ANTI_ALIASING
		{
			tsr_ushort2 EncodedInputTexelOffset = dpv_force_interleave_registers_array(RawEncodedInputTexelOffset)[0];
			
			NoiseFiltering = tsr_half2(dpv_force_interleave_registers_array(RawEncodedInputTexelOffset)[1]) * rcp(tsr_half(255.0));

			SpatialAntiAliasingOffset = DecodeSpatialAntiAliasingOffset(EncodedInputTexelOffset);
		}
		#endif
	}

	// Apply lens distortion on the reprojection.
	#if CONFIG_LENS_DISTORTION
	BRANCH
	if (bLensDistortion)
	{
		float2x2 UndistortedScreenPos = ApplyScreenTransform(float2x2(HistoryPixelPos), HistoryPixelPosToScreenPos);

		UNROLL_N(DPV_PIXEL_PER_LANE)
		for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
		{
			float2 LocalUndistortedScreenPos = dpv_access_pixel(UndistortedScreenPos, PixelId);
			float2 LocalPrevScreenPos = dpv_access_pixel(PrevScreenPos, PixelId);
			float2 LocalSrcScreenPos = ApplyDisplacementTextureOnScreenPos(PrevDistortingDisplacementTexture, LocalPrevScreenPos, LocalUndistortedScreenPos);

			//LocalSrcScreenPos = lerp(LocalUndistortedScreenPos, LocalSrcScreenPos, saturate(OutputPixelVelocity[PixelId] * tsr_half(8.0)));
			
			#if CONFIG_HISTORY_RESURRECTION
				float2 LocalSrcResurrectedScreenPos = ApplyDisplacementTextureOnScreenPos(ResurrectedDistortingDisplacementTexture, LocalPrevScreenPos, LocalUndistortedScreenPos);
				LocalSrcScreenPos = select(bIsHistoryResurrection[PixelId], LocalSrcResurrectedScreenPos, LocalSrcScreenPos);
			#endif

			PrevScreenPos[0][PixelId] = LocalSrcScreenPos.x;
			PrevScreenPos[1][PixelId] = LocalSrcScreenPos.y;
		}
	}
	#endif
	
	// Detect whether the history reprojection is successful
	bool2 bIsOffScreen;
	bool2 bIsDisoccluded;
	float Overscan = 1.0f;

#if CONFIG_LENS_DISTORTION
	Overscan = DistortionOverscan;
#endif
	
	// When there is a distortion overscan, this causes a hard edge where the distortion map ends at its original frustum. Consider
	// this edge as off screen (which it will be after being cropped at the end of the post process chain) to prevent ghosting from the edge
	IsOffScreenOrDisoccluded(
		bCameraCut,
		PrevScreenPos * Overscan,
		bIsParallaxRejected,
		/* out */ bIsOffScreen,
		/* out */ bIsDisoccluded);
		
	// Final post processing.
	#if !CONFIG_CLAMP
	{
		bIsDisoccluded = false;
		#if CONFIG_HISTORY_RESURRECTION
			bIsHistoryResurrection = false;
		#endif
		LowFrequencyRejection = tsr_half(1.0).xx;
		LowFrequencyClamp = tsr_half(1.0).xx;
	}
	#endif

	// Reproject history
	tsr_halfCx2 PrevHighFrequencyColor;
	tsr_half2 PrevHistoryValidity;

	#if CONFIG_SELECTIVE_REPROJECTION
	BRANCH
	if (all(or(LowFrequencyRejection <= tsr_half(0.0).xx, bIsOffScreen)))
	{
		PrevHighFrequencyColor = tsr_half(0.0);
		PrevHistoryValidity    = tsr_half(0.0);
	}
	else
	#endif
	PLATFORM_SPECIFIC_ISOLATE
	{
		tsr_halfC RawHighFrequency[BICUBIC_CATMULL_ROM_SAMPLES][DPV_PIXEL_PER_LANE];
		tsr_halfM RawMetadata[BICUBIC_CATMULL_ROM_SAMPLES][DPV_PIXEL_PER_LANE];
		tsr_half2 RawKernelWeight[BICUBIC_CATMULL_ROM_SAMPLES];

		// Issues texture fetches.
		PLATFORM_SPECIFIC_ISOLATE
		{
			float2x2 PrevHistoryBufferUV = ApplyScreenTransform(PrevScreenPos, ScreenPosToPrevHistoryBufferUV);
			PrevHistoryBufferUV[0] = fastClamp(PrevHistoryBufferUV[0], PrevHistoryInfo_UVViewportBilinearMin[0], PrevHistoryInfo_UVViewportBilinearMax[0]);
			PrevHistoryBufferUV[1] = fastClamp(PrevHistoryBufferUV[1], PrevHistoryInfo_UVViewportBilinearMin[1], PrevHistoryInfo_UVViewportBilinearMax[1]);


			FCatmullRomSamples Samples0 = GetBicubic2DCatmullRomSamples_Stubbe(
				dpv_lo(PrevHistoryBufferUV), PrevHistoryInfo_Extent, PrevHistoryInfo_ExtentInverse);
			FCatmullRomSamples Samples1 = GetBicubic2DCatmullRomSamples_Stubbe(
				dpv_hi(PrevHistoryBufferUV), PrevHistoryInfo_Extent, PrevHistoryInfo_ExtentInverse);

			UNROLL_N(BICUBIC_CATMULL_ROM_SAMPLES)
			for (uint i = 0; i < BICUBIC_CATMULL_ROM_SAMPLES; i++)
			{
				float2 SampleUV0 = fastClamp(Samples0.UV[i], PrevHistoryInfo_UVViewportBilinearMin, PrevHistoryInfo_UVViewportBilinearMax);
				float2 SampleUV1 = fastClamp(Samples1.UV[i], PrevHistoryInfo_UVViewportBilinearMin, PrevHistoryInfo_UVViewportBilinearMax);
					
				tsr_half2 KernelWeight = dpv_interleave_registers(tsr_half(Samples0.Weight[i]), tsr_half(Samples1.Weight[i]));

				RawKernelWeight[i] = KernelWeight;

				UNROLL_N(2)
				for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
				{
					float2 SampleUV = PixelId == 0 ? SampleUV0 : SampleUV1;
						
					#if CONFIG_HISTORY_RESURRECTION
						float FrameIndex = select(bIsHistoryResurrection[PixelId], ResurrectionFrameIndex, PrevFrameIndex);
					#else
						float FrameIndex = PrevFrameIndex;
					#endif

					RawHighFrequency[i][PixelId] = BilinearSampleColorHistory(PrevHistoryColorTexture, float3(SampleUV, FrameIndex));
					RawMetadata[i][PixelId]      = PrevHistoryMetadataTexture.SampleLevel(GlobalBilinearClampedSampler, float3(SampleUV, FrameIndex), 0);
				}

			} // for (uint i = 0; i < BICUBIC_CATMULL_ROM_SAMPLES; i++)
		}
		
		#if CONFIG_HISTORY_RESURRECTION
			const tsr_half2 PreExposureCorrection = select(
				bIsHistoryResurrection,
				tsr_half(ResurrectionPreExposureCorrection),
				tsr_half(HistoryPreExposureCorrection));
		#else
			const tsr_half2 PreExposureCorrection = tsr_half(HistoryPreExposureCorrection);
		#endif

		// Process history texture fetches.
		tsr_halfCx2 AccumulateHighFrequency = tsr_half(0);
		tsr_halfMx2 AccumulateMetadata = tsr_half(0);

		PLATFORM_SPECIFIC_ISOLATE
		{
			UNROLL_N(BICUBIC_CATMULL_ROM_SAMPLES)
			for (uint i = 0; i < BICUBIC_CATMULL_ROM_SAMPLES; i++)
			{
				// TODO: Should use dpv_force_interleave_registers_array() but there is a shader compiler bug
				tsr_halfCx2 SampleHighFrequency = dpv_interleave_registers_array(RawHighFrequency[i]);
				tsr_halfMx2 SampleMetadata = dpv_interleave_registers_array(RawMetadata[i]);

				tsr_half2 KernelWeight = RawKernelWeight[i];

				AccumulateHighFrequency[0] += SampleHighFrequency[0] * (KernelWeight * PreExposureCorrection);
				AccumulateHighFrequency[1] += SampleHighFrequency[1] * (KernelWeight * PreExposureCorrection);
				AccumulateHighFrequency[2] += SampleHighFrequency[2] * (KernelWeight * PreExposureCorrection);
				#if CONFIG_SCENE_COLOR_ALPHA
					AccumulateHighFrequency[3] += SampleHighFrequency[3] * KernelWeight;
				#endif
				AccumulateMetadata  += dpv_scale(SampleMetadata, KernelWeight);
			}
		}

		// Corrects history.
		{
			// Super bight highlights have a tendy to generate negatives
			bool2 bHasAnyNegativeOrNaN = or(AccumulateHighFrequency[0] < 0.0, or(AccumulateHighFrequency[1] < 0.0, AccumulateHighFrequency[2] < 0.0));
			#if CONFIG_SCENE_COLOR_ALPHA
				bHasAnyNegativeOrNaN = or(bHasAnyNegativeOrNaN, AccumulateHighFrequency[3] < 0.0);
			#endif

			BRANCH
			if (any(bHasAnyNegativeOrNaN))
			{
				AccumulateHighFrequency[0] = select(bHasAnyNegativeOrNaN, dpv_interleave_registers_array(RawHighFrequency[2])[0] * PreExposureCorrection, AccumulateHighFrequency[0]);
				AccumulateHighFrequency[1] = select(bHasAnyNegativeOrNaN, dpv_interleave_registers_array(RawHighFrequency[2])[1] * PreExposureCorrection, AccumulateHighFrequency[1]);
				AccumulateHighFrequency[2] = select(bHasAnyNegativeOrNaN, dpv_interleave_registers_array(RawHighFrequency[2])[2] * PreExposureCorrection, AccumulateHighFrequency[2]);
				#if CONFIG_SCENE_COLOR_ALPHA
					AccumulateHighFrequency[3] = select(bHasAnyNegativeOrNaN, dpv_interleave_registers_array(RawHighFrequency[2])[3], AccumulateHighFrequency[3]);
				#endif
			
				#if CONFIG_SCENE_COLOR_ALPHA
					AccumulateHighFrequency = -dpv_min(-AccumulateHighFrequency, tsr_half(0.0).xxxx);
				#else
					AccumulateHighFrequency = -dpv_min(-AccumulateHighFrequency, tsr_half(0.0).xxx);
				#endif
			}

			AccumulateMetadata = -dpv_min(-AccumulateMetadata, tsr_half(0.0).xx);
		}
		
		// Unpack history.
		{
			PrevHighFrequencyColor = AccumulateHighFrequency;
			#if CONFIG_METADATA_CHANNELS == 1
				PrevHistoryValidity = AccumulateMetadata;
			#else
				PrevHistoryValidity = AccumulateMetadata[0];
			#endif
		}
	}
	
	// TODO: CONFIG_MANUAL_LDS_SPILL

	// Filter input scene color at predictor frequency.
	tsr_halfCx2 FilteredInputColor;
	tsr_halfCx2 InputMinColor;
	tsr_halfCx2 InputMaxColor;

	tsr_half2 InputPixelAlignement;
	
	tsr_half2 BlendClamp;
	tsr_half2 PrevWeight;
	tsr_half2 CurrentWeight;
	
	PLATFORM_SPECIFIC_ISOLATE
	{
		tsr_halfC RawInputColorArray[CONFIG_SAMPLES_COUNT][DPV_PIXEL_PER_LANE];

		tsr_half2x2 RawdPPArray[CONFIG_SAMPLES_COUNT];
		tsr_half2   RawSampleSpatialWeightArray[CONFIG_SAMPLES_COUNT];
		bool2       bIsDisabledSampleArray[CONFIG_SAMPLES_COUNT];
		
		// Issues overlapped texture fetches
		PLATFORM_SPECIFIC_ISOLATE
		{
			// Detect if HistoryBufferUV would be outside of the viewport.
			tsr_half2 SpatialAntiAliasingLerp = select(or(bIsOffScreen, and(bIsDisoccluded, !bIsHistoryResurrection)), tsr_half(1.0).xx, saturate(tsr_half(1.0) - LowFrequencyRejection * tsr_half(4.0)));

			#if CONFIG_REJECTION_ANTI_ALIASING
			{
				InputPPCo += dpv_scale(SpatialAntiAliasingOffset, SpatialAntiAliasingLerp);

				InputPPCk = floor(InputPPCo) + 0.5;
				InputPixelPos = ClampPixelOffset(
					tsr_short2x2(InputPPCo),
					InputPixelPosMin, InputPixelPosMax);
			}
			#endif
			
			// Vector in pixel between pixel K -> O.
			tsr_half2x2 dInputKO = tsr_half2x2(InputPPCo - InputPPCk);
			
			// Compute upscaling kernel size based of the rejections and number of samples already in history.
			tsr_half2 KernelInputToHistoryFactor;
			{
				const tsr_half ActualHistorySampleCount = tsr_half(HistorySampleCount);
				const tsr_half ActualHistoryHisteresis = tsr_half(HistoryHisteresis);

				tsr_half2 MinRejectionBlendFactor = RejectionFactorToBlendFactor(min(LowFrequencyRejection, ReprojectionUpscaleCorrection));
				MinRejectionBlendFactor = select(bIsOffScreen, tsr_half(1.0).xx, MinRejectionBlendFactor);

				tsr_half2 CoarseInputPixelAlignement = ComputeSampleWeigth(tsr_half(1.0), dInputKO, /* MinimalContribution = */ float(0.0));
				tsr_half2 IdealInputPixelAlignement = ComputeSampleWeigth(tsr_half(InputToHistoryFactor), dInputKO, /* MinimalContribution = */ float(0.0));
			
				tsr_half2 CoarseCurrentContribution = CoarseInputPixelAlignement * tsr_half(ActualHistoryHisteresis);
				tsr_half2 IdealCurrentContribution = IdealInputPixelAlignement * tsr_half(ActualHistoryHisteresis);

				// Reduces the number of pixels when rejecting history
				tsr_half2 ClampedPrevHistoryValidity = PrevHistoryValidity;
				#if 1
				{
					tsr_half2 MaxValidity = tsr_half(1.0) - tsr_half(WeightClampingRejection) * DecreaseValidityMultiplier;

					ClampedPrevHistoryValidity = min(ClampedPrevHistoryValidity, MaxValidity);
				}
				#endif

				const tsr_half MaxWeight = tsr_half(1.0);
	
				// Compute the histeresis if was refining the history
				tsr_half2 RefiningHisteresis = select(
					ClampedPrevHistoryValidity + IdealCurrentContribution > tsr_half(0.0),
					IdealCurrentContribution * rcp(ClampedPrevHistoryValidity + IdealCurrentContribution),
					tsr_half(1.0));
				
				// Compute the prev weight with coarse pixel alignment when rejecting the history.
				tsr_half2 CoarseRejectedPrevWeight = min(
					CoarseCurrentContribution * ComputePrevWeightMultiplier(MinRejectionBlendFactor),
					MaxWeight);
					
				// Compute the prev weight with coarse pixel alignment when refining.
				tsr_half2 CoarseRefiningPrevWeight = min(
					CoarseCurrentContribution * ComputePrevWeightMultiplier(RefiningHisteresis),
					MaxWeight);
				
				// Do not refine when rejected previous weight is lower than refining with coarse pixel alignment.
				// bIsNotRefining = CoarseRejectedPrevWeight < CoarseRefiningPrevWeight
				tsr_half2 bIsRefining = select(CoarseRejectedPrevWeight < CoarseRefiningPrevWeight, tsr_half(0.0), tsr_half(1.0));
				bIsRefining = min(bIsRefining, saturate(ClampedPrevHistoryValidity * ActualHistorySampleCount));
	
				// Soften the spatial kernel when there is high noise in the input to make parallax disocclusion of detailed geometry less distracting.
				tsr_half2 OffScreenInputToHistoryFactor = tsr_half(1.0) - tsr_half(0.5) * NoiseFiltering;

				tsr_half2 KernelInputToHistoryLerp = select(or(bIsOffScreen, and(bIsDisoccluded, !bIsHistoryResurrection)), tsr_half(0.0).xx, saturate(LowFrequencyRejection * tsr_half(16.0) - tsr_half(13.0)) * bIsRefining);
				KernelInputToHistoryFactor = lerp(OffScreenInputToHistoryFactor, tsr_half(InputToHistoryFactor), KernelInputToHistoryLerp);
				tsr_half2 KernelInputToHistoryAlignmentFactor = lerp(tsr_half(1.0).xx, tsr_half(InputToHistoryFactor), KernelInputToHistoryLerp);
			
				InputPixelAlignement = ComputeSampleWeigth(KernelInputToHistoryAlignmentFactor, dInputKO, /* MinimalContribution = */ float(0.0));
			
				CurrentWeight = InputPixelAlignement * tsr_half(ActualHistoryHisteresis);
				PrevWeight = min(select(CurrentWeight > tsr_half(0.0), CurrentWeight, CoarseCurrentContribution) * ComputePrevWeightMultiplier(MinRejectionBlendFactor), ClampedPrevHistoryValidity);
				PrevWeight = min(PrevWeight, MaxWeight - CurrentWeight);	
			}

			UNROLL_N(CONFIG_SAMPLES_COUNT)
			for (uint SampleId = 0; SampleId < CONFIG_SAMPLES_COUNT; SampleId++)
			{
				tsr_short2x2 InputSamplePixelPos;
				tsr_half2x2 InputPixelOffset;
				bool2 bIsDuplicatedSample;
				bool2 bIsDisabledSample;
				ComputeInputKernelSamplePosition(
					InputPixelPos, dInputKO, SampleId,
					InputPixelPosMin, InputPixelPosMax,
					/* out */ InputSamplePixelPos,
					/* out */ InputPixelOffset,
					/* out */ bIsDuplicatedSample,
					/* out */ bIsDisabledSample);
				
				tsr_half2x2 dPP = InputPixelOffset - dInputKO;
				tsr_half2 SampleSpatialWeight = ComputeSampleWeigth(KernelInputToHistoryFactor, dPP, /* MinimalContribution = */ float(0.005));
				SampleSpatialWeight = select(bIsDuplicatedSample, tsr_half(0.0), SampleSpatialWeight);

				bIsDisabledSampleArray[SampleId]      = bIsDisabledSample;
				RawdPPArray[SampleId]                 = dPP; 
				RawSampleSpatialWeightArray[SampleId] = SampleSpatialWeight;

				UNROLL_N(DPV_PIXEL_PER_LANE)
				for (uint PixelId = 0; PixelId < DPV_PIXEL_PER_LANE; PixelId++)
				{
					RawInputColorArray[SampleId][PixelId] = InputSceneColorTexture[dpv_access_pixel(InputSamplePixelPos, PixelId)];
				}
			} // for (uint SampleId = 0; SampleId < CONFIG_SAMPLES_COUNT; SampleId++)
		}
			
		tsr_halfCx2 InputColorCenter = dpv_force_interleave_registers_array(RawInputColorArray[0]);

		tsr_half2 FilteredInputColorWeight = tsr_half(0.0);
		FilteredInputColor = tsr_half(0.0);
		InputMinColor = InputColorCenter;
		InputMaxColor = InputColorCenter;

		UNROLL_N(CONFIG_SAMPLES_COUNT)
		for (uint SampleId = 0; SampleId < CONFIG_SAMPLES_COUNT; SampleId++)
		{
			bool2 bIsDisabledSample = bIsDisabledSampleArray[SampleId];
			tsr_half2 SampleSpatialWeight = RawSampleSpatialWeightArray[SampleId];
			tsr_halfCx2 InputColor = dpv_force_interleave_registers_array(RawInputColorArray[SampleId]);

			tsr_half2 ToneWeight = HdrWeight4(InputColor);

			FilteredInputColor       += dpv_scale((SampleSpatialWeight * ToneWeight), InputColor);
			FilteredInputColorWeight += (SampleSpatialWeight * ToneWeight);

			if (SampleId != 0)
			{
				tsr_halfCx2 ClampBoxSample;
				ClampBoxSample[0] = select(bIsDisabledSample, InputColorCenter[0], InputColor[0]);
				ClampBoxSample[1] = select(bIsDisabledSample, InputColorCenter[1], InputColor[1]);
				ClampBoxSample[2] = select(bIsDisabledSample, InputColorCenter[2], InputColor[2]);
				#if CONFIG_SCENE_COLOR_ALPHA
					ClampBoxSample[3] = select(bIsDisabledSample, InputColorCenter[3], InputColor[3]);
				#endif
			
				InputMinColor = min(InputMinColor, ClampBoxSample);
				InputMaxColor = max(InputMaxColor, ClampBoxSample);
			}
		} // // for (uint SampleId = 0; SampleId < CONFIG_SAMPLES_COUNT; SampleId++)
		
		FilteredInputColor = dpv_scale(FilteredInputColor, rcp(FilteredInputColorWeight));

		#if CONFIG_SCENE_COLOR_OVERFLOW
		{
			#if CONFIG_SCENE_COLOR_ALPHA
				FilteredInputColor = min(FilteredInputColor, dpv_interleave_mono_registers(LargestSceneColorRGBA));
			#else
				FilteredInputColor = min(FilteredInputColor, dpv_interleave_mono_registers(LargestSceneColorRGB));
			#endif
		}
		#endif
	}
		
	// Contribute current frame input into the predictor for next frame.
	tsr_halfCx2 FinalHighFrequencyColor;
	tsr_half2 FinalHistoryValidity;

	{
		tsr_halfCx2 BlendedPrevHighFrequencyColor;
		{
			tsr_halfCx2 ClampedPrevHighFrequencyColor = fastClamp(PrevHighFrequencyColor, InputMinColor, InputMaxColor);
			tsr_half2x2 Weight = WeightedLerpFactors(HdrWeight4(ClampedPrevHighFrequencyColor), HdrWeight4(PrevHighFrequencyColor), DisableHistoryClamp);
			BlendedPrevHighFrequencyColor = dpv_scale(ClampedPrevHighFrequencyColor, Weight[0]) + dpv_scale(PrevHighFrequencyColor, Weight[1]);
		}
		
		// Clamp the validity due to motion to maintain better sharpness in history reprojection under motion.
		#if 1
		{
			tsr_half2 MaxValidity = tsr_half(1.0) - tsr_half(WeightClampingPixelSpeedAmplitude) * saturate(OutputPixelVelocity * tsr_half(InvWeightClampingPixelSpeed));

			// Clamp up the max validity to favor stability under motion on high contrast edges.
			#if 1
			{
				tsr_half2 PrevHistoryLuma = Luma4(BlendedPrevHighFrequencyColor);
				tsr_half2 FilteredLuma = Luma4(FilteredInputColor);

				tsr_half2 MinValidityForStability = abs(FilteredLuma - PrevHistoryLuma) / max(FilteredLuma, PrevHistoryLuma);
			
				MaxValidity = max(MaxValidity, MinValidityForStability);
			}
			#endif
			
			PrevWeight = min(PrevWeight, MaxValidity);
		}
		#endif

		tsr_half2 OutputValidity = CurrentWeight + PrevWeight;

		tsr_half2 PrevHistoryToneWeight = HdrWeightY(Luma4(BlendedPrevHighFrequencyColor));
		tsr_half2 FilteredInputToneWeight = HdrWeight4(FilteredInputColor);

		// NOTE: weight calculation is intentionally done at full float precision to 
		// avoid quantization errors in cases of extreme difference.
			
		float2 BlendPrevHistory = PrevWeight * PrevHistoryToneWeight;
		float2 BlendFilteredInput = CurrentWeight * FilteredInputToneWeight;

		float2 CommonWeight = SafeRcp(BlendPrevHistory + BlendFilteredInput);

		FinalHighFrequencyColor = (
			dpv_scale(BlendedPrevHighFrequencyColor, tsr_half2(CommonWeight * BlendPrevHistory)) +
			dpv_scale(FilteredInputColor, tsr_half2(CommonWeight * BlendFilteredInput)));
				
		// Quantize validity for the 8bit encoding to avoid numerical shift between color and validity.
		FinalHistoryValidity = ceil(tsr_half(255.0) * OutputValidity) * rcp(tsr_half(255.0));
	}

	PLATFORM_SPECIFIC_ISOLATE
	{
		#if CONFIG_SCENE_COLOR_ALPHA
			const tsr_halfC ColorNull = tsr_half(0.0).xxxx;
			const tsr_halfC ColorMax10BitsFloat = tsr_half(Max10BitsFloat).xxxx;
		#else
			const tsr_halfC ColorNull = tsr_half(0.0).xxx;
			const tsr_halfC ColorMax10BitsFloat = tsr_half(Max10BitsFloat).xxx;
		#endif

		uint LocalGroupThreadIndex = GetGroupThreadIndex(GroupThreadIndex, GroupWaveIndex);

		#if 1
			tsr_short2x2 LocalHistoryPixelPos = dpv_add(
				tsr_short2(GroupId) * tsr_short2(TILE_SIZE, TILE_SIZE),
				tsr_short2x2(Map8x8Tile2x2LaneDPV(LocalGroupThreadIndex)));
		#else
			tsr_short2x2 LocalHistoryPixelPos = HistoryPixelPos;
		#endif
			
		LocalHistoryPixelPos = InvalidateOutputPixelPos(LocalHistoryPixelPos, HistoryInfo_ViewportMax);
		
		#if CONFIG_METADATA_CHANNELS == 1
			tsr_halfMx2 FinalMetadata = FinalHistoryValidity;
		#else
			tsr_halfMx2 FinalMetadata;
			FinalMetadata[0] = FinalHistoryValidity;
		#endif

		// Stochastically round up or down using the hardware RWTexture2D truncation unit to take into precision
		// loss due to pixel format encoding.
		#if CONFIG_ENABLE_STOCASTIC_QUANTIZATION
		{
			uint Random = Rand3DPCG16(int3(dpv_lo(LocalHistoryPixelPos), View.StateFrameIndexMod8)).x;
			tsr_half E = tsr_half(Rand16ToFloat(Random));
			
			FinalHighFrequencyColor = QuantizeForFloatRenderTarget(FinalHighFrequencyColor, E, HistoryQuantizationError);
		}
		#endif

		// Protect from NaN and +Inf when writing out the history.
		{
			FinalHighFrequencyColor = -dpv_min(-FinalHighFrequencyColor, ColorNull);
			FinalHighFrequencyColor = dpv_min(FinalHighFrequencyColor, ColorMax10BitsFloat);
		}

		// Ensure that alpha values that are expected to be opaque (but are only close to opaque) are forced to be opaque.
		// (0.995 chosen to accommodate handling of 254/255)
		#if CONFIG_SCENE_COLOR_ALPHA
		{
			FinalHighFrequencyColor[3] = select(FinalHighFrequencyColor[3] > tsr_half(0.995), tsr_half(1.0), FinalHighFrequencyColor[3]);
			FinalHighFrequencyColor[3] = select(FinalHighFrequencyColor[3] < tsr_half(0.005), tsr_half(0.0), FinalHighFrequencyColor[3]);
		}
		#endif

		// Output full res history
		{
			// Output final history lo pixel.
			{
				HistoryColorOutput[tsr_short3(dpv_lo(LocalHistoryPixelPos), HistoryArrayIndices_HighFrequency)] = dpv_lo(FinalHighFrequencyColor);
				HistoryMetadataOutput[tsr_short3(dpv_lo(LocalHistoryPixelPos), 0)]  = dpv_lo(FinalMetadata);
			}
		
			// Output final history hi pixel.
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
			{
				HistoryColorOutput[tsr_short3(dpv_hi(LocalHistoryPixelPos), HistoryArrayIndices_HighFrequency)] = dpv_hi(FinalHighFrequencyColor);
				HistoryMetadataOutput[tsr_short3(dpv_hi(LocalHistoryPixelPos), 0)] = dpv_hi(FinalMetadata);
			}
			#endif // !CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
		}

		// Output final scene color Mip1
		{
			tsr_halfCx2 HalfResOutput = FinalHighFrequencyColor;
			tsr_short2x2 HalfResOutputPixelPos = dpv_interleave_mono_registers(tsr_short(-1).xx);

			BRANCH
			if (bGenerateOutputMip1 || bGenerateOutputMip2 || bGenerateOutputMip3)
			{
				HalfResOutput = DownsampleSceneColor(HalfResOutput, LocalGroupThreadIndex, /* XorButterFly = */ uint2(0x01, 0x02));
				
				BRANCH
				if (bGenerateOutputMip3)
				{
					HalfResOutput = DownsampleSceneColor(HalfResOutput, LocalGroupThreadIndex, /* XorButterFly = */ uint2(0x04, 0x10));
					HalfResOutput = DownsampleSceneColor(HalfResOutput, LocalGroupThreadIndex, /* XorButterFly = */ uint2(0x08, 0x20));

					HalfResOutputPixelPos[0] = (LocalHistoryPixelPos[0] >> tsr_short(3)) | (((LocalHistoryPixelPos[0] & tsr_short(0x7))) * tsr_short(~0));
					HalfResOutputPixelPos[1] = (LocalHistoryPixelPos[1] >> tsr_short(3)) | (((LocalHistoryPixelPos[1] & tsr_short(0x7))) * tsr_short(~0));
				}
				else if (bGenerateOutputMip2)
				{
					HalfResOutput = DownsampleSceneColor(HalfResOutput, LocalGroupThreadIndex, /* XorButterFly = */ uint2(0x04, 0x10));

					HalfResOutputPixelPos[0] = (LocalHistoryPixelPos[0] >> tsr_short(2)) | (((LocalHistoryPixelPos[0] & tsr_short(0x3))) * tsr_short(~0));
					HalfResOutputPixelPos[1] = (LocalHistoryPixelPos[1] >> tsr_short(2)) | (((LocalHistoryPixelPos[1] & tsr_short(0x3))) * tsr_short(~0));
				}
				else
				{
					HalfResOutputPixelPos[0] = (LocalHistoryPixelPos[0] >> tsr_short(1)) | (((LocalHistoryPixelPos[0] & tsr_short(0x1))) * tsr_short(~0));
					HalfResOutputPixelPos[1] = (LocalHistoryPixelPos[1] >> tsr_short(1)) | (((LocalHistoryPixelPos[1] & tsr_short(0x1))) * tsr_short(~0));
				}
			}

			SceneColorOutputMip1[tsr_short3(dpv_lo(HalfResOutputPixelPos), 0)] = dpv_lo(HalfResOutput);
			
			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				SceneColorOutputMip1[tsr_short3(dpv_hi(HalfResOutputPixelPos), 0)] = dpv_hi(HalfResOutput);
			#endif
		}
		
		#if DEBUG_OUTPUT
		for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
		{
			DebugOutput[tsr_short3(dpv_lo(LocalHistoryPixelPos), DebugId)] = dpv_lo(Debug[DebugId]);

			#if CONFIG_ENABLE_DUAL_PIXEL_VECTORIZATION
				DebugOutput[tsr_short3(dpv_hi(LocalHistoryPixelPos), DebugId)] = dpv_hi(Debug[DebugId]);
			#endif
		}
		#endif
	}
}
