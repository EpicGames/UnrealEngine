// Copyright Epic Games, Inc. All Rights Reserved.

#include "TSRColorSpace.ush"
#include "TSRReprojectionField.ush"
#if TSR_SUPPORT_LENS_DISTORTION
	#include "../LensDistortion.ush"
#endif


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8


//------------------------------------------------------- CONSTS

static tsr_half3 kWhite  = tsr_half3(0.33, 0.33, 0.33);
static tsr_half3 kYellow = tsr_half3(1.00, 1.00, 0.10);
static tsr_half3 kRed    = tsr_half3(1.00, 0.10, 0.10);
static tsr_half3 kOrange = tsr_half3(1.00, 0.75, 0.10);
static tsr_half3 kGreen  = tsr_half3(0.10, 1.00, 0.10);
static tsr_half3 kBlue   = tsr_half3(0.10, 0.10, 1.00);
static tsr_half3 kCyan   = tsr_half3(0.10, 1.00, 1.00);
static tsr_half3 kPink   = tsr_half3(1.00, 0.10, 0.50);


//------------------------------------------------------- PARAMETERS

FScreenTransform OutputPixelPosToScreenPos;
FScreenTransform ScreenPosToHistoryUV;
FScreenTransform ScreenPosToInputPixelPos;
FScreenTransform ScreenPosToInputUV;
FScreenTransform ScreenPosToMoireHistoryUV;
float2 MoireHistoryUVBilinearMin;
float2 MoireHistoryUVBilinearMax;
float4x4 ClipToResurrectionClip;
uint2 OutputViewRectMin;
uint2 OutputViewRectMax;
uint VisualizeId;
uint bCanResurrectHistory;
uint bCanSpatialAntiAlias;
uint bReprojectionField;
float MaxHistorySampleCount;
float OutputToHistoryResolutionFractionSquare;
float FlickeringFramePeriod;
float ExposureOffsetFactor;

Texture2D<float2>    PrevDistortingDisplacementTexture;
Texture2D<float2>    ResurrectedDistortingDisplacementTexture;
Texture2D<float2>    UndistortingDisplacementTexture;
Texture2D<tsr_half3> InputTexture;
Texture2D<tsr_half>  InputMoireLumaTexture;
Texture2D<tsr_half4> InputSceneTranslucencyTexture;
Texture2D<tsr_half3> SceneColorTexture;
Texture2D<float2>    ClosestDepthTexture;
Texture2D<uint>      ReprojectionBoundaryTexture;
Texture2D<uint>      ReprojectionJacobianTexture;
Texture2D<uint>      ReprojectionVectorTexture;
Texture2D<tsr_ushort> IsMovingMaskTexture;
Texture2D<tsr_ushort> ThinGeometryTexture;
Texture2D<tsr_half2> DecimateMaskTexture;
Texture2D<tsr_half4> HistoryRejectionTexture;
Texture2D<tsr_half4> MoireHistoryTexture;
Texture2D<tsr_ushort> AntiAliasMaskTexture;
Texture2D<tsr_half4> HistoryMetadataTexture;
Texture2D<tsr_half3> ResurrectedHistoryColorTexture;

RWTexture2D<tsr_half4> Output;


//------------------------------------------------------- DEBUG

static float4 Debug;
	

//------------------------------------------------------- FUNCTION

#if !TSR_SUPPORT_LENS_DISTORTION
float2 ApplyLensDistortionOnScreenPos(Texture2D<float2> DisplacementTexture, float2 DestScreenPos)
{
	return DestScreenPos;
}
#endif

tsr_half3 GrayScaleColor(tsr_half3 SceneColor)
{
	tsr_half3 C = SceneColor;
	C.rgb = dot(tsr_half(0.3333).xxx, SceneColor.rgb);
	return C;
}

struct FContext
{
	tsr_short2 OutputPixelPos;
	float2 ScreenPos;
	float2 InputUV;
	tsr_short2 InputPixelPos;
};

tsr_half3 SampleTSROutputColor(FContext Ctx)
{
	return SceneColorTexture[Ctx.OutputPixelPos];
}

tsr_half3 SampleTSROutputGray(FContext Ctx)
{
	return GrayScaleColor(SceneColorTexture[Ctx.OutputPixelPos]);
}

struct FReprojectionFieldSample
{
	bool bIsOffScreen;
	bool bIsFullDilate;
	bool bAntiAliasReprojectionField;
	tsr_short2 Offset;
	tsr_half2 Boundary;
	float2 Vector;
	float2 PrevScreenPos;
	tsr_half2x2 Jacobian;
};

FReprojectionFieldSample SampleReprojectionField(FContext Ctx)
{
	FReprojectionFieldSample Reprojection;

	uint EncodedReprojectionBoundary = ReprojectionBoundaryTexture[Ctx.InputPixelPos];
	Reprojection.Offset = DecodeReprojectionOffset(EncodedReprojectionBoundary);
	Reprojection.Boundary = DecodeReprojectionBoundary(EncodedReprojectionBoundary);
	Reprojection.bIsFullDilate = all(Reprojection.Boundary == kFullDilateBoundary);
	Reprojection.bAntiAliasReprojectionField = any(Reprojection.Offset != tsr_short(0)) && !Reprojection.bIsFullDilate;

	uint EncodedReprojectionVector   = ReprojectionVectorTexture[Ctx.InputPixelPos + Reprojection.Offset];
	uint EncodedReprojectionJacobian = ReprojectionJacobianTexture[Ctx.InputPixelPos + Reprojection.Offset];

	Reprojection.Vector   = DecodeReprojectionVector(EncodedReprojectionVector);
	Reprojection.Jacobian = DecodeReprojectionJacobian(EncodedReprojectionJacobian);
	
	Reprojection.PrevScreenPos = Ctx.ScreenPos - Reprojection.Vector;

	// Detect whether the history reprojection is successful
	bool2 bIsOffScreenLocal;
	bool2 bIsDisoccluded;
	IsOffScreenOrDisoccluded(
		bCameraCut,
		dpv_interleave_mono_registers(Reprojection.PrevScreenPos),
		/* bIsParallaxRejected = */ false,
		/* out */ bIsOffScreenLocal,
		/* out */ bIsDisoccluded);

	Reprojection.bIsOffScreen = bIsOffScreenLocal[0];

	return Reprojection;
}

struct FDecimateMaskSample
{
	bool bIsOffScreen;
	bool bIsParallaxDisocclusion;
	bool bIsDisoccluded;
	bool bHasPixelAnimation;
	bool bDisableAnimationAA;
	bool bReprojectionHollFill;
};

FDecimateMaskSample SampleDecimateMask(FContext Ctx)
{
	FDecimateMaskSample Decimate;

	tsr_half2 DecimateMask = DecimateMaskTexture[Ctx.InputPixelPos];
	
	tsr_ushort BitMask = tsr_ushort(DecimateMask.r * tsr_half(255.0));
	Decimate.bIsOffScreen            = (BitMask & tsr_ushort(0x01)) != tsr_ushort(0);
	Decimate.bIsParallaxDisocclusion = (BitMask & tsr_ushort(0x02)) != tsr_ushort(0);
	Decimate.bIsDisoccluded          = (BitMask & tsr_ushort(0x03)) != tsr_ushort(0);
	Decimate.bHasPixelAnimation      = (BitMask & tsr_ushort(0x04)) != tsr_ushort(0);
	Decimate.bReprojectionHollFill   = (BitMask & tsr_ushort(0x08)) != tsr_ushort(0);
	Decimate.bDisableAnimationAA	 = (BitMask & tsr_ushort(0x20)) != tsr_ushort(0);
	return Decimate;
}


//------------------------------------------------------- ENTRY POINT

tsr_half3 DisplayHistorySampleCount(FContext Ctx)
{
	float2 HistoryUV =  ApplyScreenTransform(Ctx.ScreenPos, ScreenPosToHistoryUV);
	HistoryUV = fastClamp(HistoryUV, HistoryInfo_UVViewportBilinearMin, HistoryInfo_UVViewportBilinearMax);

	tsr_half Validity = HistoryMetadataTexture.SampleLevel(GlobalBilinearClampedSampler, HistoryUV, 0).x;

#if 0
	tsr_half SampleCount = Validity * MaxHistorySampleCount;

	const tsr_half Threshold = tsr_half(1.0) + OutputToHistoryResolutionFractionSquare;

	if (SampleCount > Threshold)
	{
		return SampleTSROutputGray(Ctx) * lerp(kOrange, kGreen, saturate((SampleCount - Threshold) / (MaxHistorySampleCount - Threshold)));
	}

	return SampleTSROutputGray(Ctx) * lerp(kRed, kOrange, saturate((SampleCount - 1.0) / (Threshold - 1.0)));
#else
	return SampleTSROutputGray(Ctx) * lerp(kRed, kGreen, saturate(Validity));
#endif
}

tsr_half3 DisplayThinGeometry(FContext Ctx)
{
	tsr_ushort EncodedThinGeometryFlag = ThinGeometryTexture[Ctx.InputPixelPos];
	bool bClusterHole = bool(EncodedThinGeometryFlag & 0x1u);
	tsr_half HistoryRelaxation = saturate(tsr_half(EncodedThinGeometryFlag >> 0x1u)*rcp(tsr_half(127.0)));
	bool bUseV2Rectifier = HistoryRelaxation > tsr_half(1.0 / 127.0);
	tsr_half3 Color = select(bClusterHole, kWhite, kRed);
	Color = lerp(kWhite,    Color, saturate((HistoryRelaxation - 0.0) * 2.0));
	Color = lerp(Color,    kGreen, saturate(tsr_half(select(bClusterHole && bUseV2Rectifier, 0.5f, 0.0f))));
	Color = lerp(Color,   kYellow, saturate(tsr_half(select(bClusterHole && !bUseV2Rectifier, 0.5f, 0.0f))));
	Color = select(bCameraCut == 0, Color, kPink);

	return SampleTSROutputColor(Ctx) * Color;   
}

tsr_half3 DisplayHistoryRejection(FContext Ctx)
{
	tsr_half LowFrequencyRejection = HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).r;

	tsr_half3 Color = kWhite;
	Color = lerp(kOrange, Color, saturate((LowFrequencyRejection - 0.5) * 2.0));
	Color = lerp(kRed,    Color, saturate((LowFrequencyRejection - 0.0) * 2.0));
	Color = select(bCameraCut == 0, Color, kPink);

	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayHistoryClamp(FContext Ctx)
{
	tsr_half LowFrequencyClamp = HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).g;

	tsr_half3 Color = kWhite;
	Color = lerp(kOrange, Color, saturate((LowFrequencyClamp - 0.5) * 2.0));
	Color = lerp(kRed,    Color, saturate((LowFrequencyClamp - 0.0) * 2.0));
	Color = select(bCameraCut == 0, Color, kPink);

	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayParallaxDisocclusionMask(FContext Ctx)
{
	FDecimateMaskSample Decimate = SampleDecimateMask(Ctx);
	
	tsr_half3 Color;
	Color = select(Decimate.bReprojectionHollFill && Decimate.bIsParallaxDisocclusion, kOrange, kWhite);
	Color = select(!Decimate.bReprojectionHollFill && Decimate.bIsParallaxDisocclusion, kRed, Color);
	Color = select(Decimate.bIsOffScreen, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayResurrectionMask(FContext Ctx)
{
	if (!bCanResurrectHistory)
	{
		return SampleTSROutputGray(Ctx) * kPink;
	}

	tsr_half LowFrequencyRejection = HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).r;
	float ClosestDeviceZ = ClosestDepthTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).g;
	bool ResurrectionMask = (tsr_ushort(HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).a * tsr_half(255.0)) & tsr_ushort(0x2)) != tsr_ushort(0x0);

	float4 ThisClip = float4(ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, Ctx.ScreenPos), ClosestDeviceZ, 1);
	float4 ResurrectionClip = mul(ThisClip, ClipToResurrectionClip); 
	float2 ResurrectionScreen = ApplyLensDistortionOnScreenPos(ResurrectedDistortingDisplacementTexture, ResurrectionClip.xy / ResurrectionClip.w);

	tsr_half3 Color = kWhite;
	Color = select(any(ResurrectionScreen != clamp(ResurrectionScreen, -1.0, 1.0)), kPink, Color);
	Color = select(ResurrectionMask, lerp(kRed, kGreen, LowFrequencyRejection), Color);

	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayResurrectedColor(FContext Ctx)
{
	if (!bCanResurrectHistory)
	{
		return SampleTSROutputGray(Ctx) * kPink;
	}

	tsr_half LowFrequencyRejection = HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).r;
	float ClosestDeviceZ = ClosestDepthTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).g;
	bool ResurrectionMask = (tsr_ushort(HistoryRejectionTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0).a * tsr_half(255.0)) & tsr_ushort(0x2)) != tsr_ushort(0x0);

	float4 ThisClip = float4(ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, Ctx.ScreenPos), ClosestDeviceZ, 1);
	float4 ResurrectionClip = mul(ThisClip, ClipToResurrectionClip); 
	float2 ResurrectionScreen = ApplyLensDistortionOnScreenPos(ResurrectedDistortingDisplacementTexture, ResurrectionClip.xy / ResurrectionClip.w);

	float2 KernelHistoryBufferUV = ApplyScreenTransform(ResurrectionScreen, ScreenPosToPrevHistoryBufferUV);
	KernelHistoryBufferUV = fastClamp(KernelHistoryBufferUV, PrevHistoryInfo_UVViewportBilinearMin, PrevHistoryInfo_UVViewportBilinearMax);

	tsr_half3 ResurrectedColor = ResurrectedHistoryColorTexture.SampleLevel(GlobalBilinearClampedSampler, KernelHistoryBufferUV, 0) * tsr_half(ResurrectionPreExposureCorrection);
	
	tsr_half3 Color = kWhite;
	Color = select(any(ResurrectionScreen != clamp(ResurrectionScreen, -1.0, 1.0)), kPink, Color);
	Color = select(ResurrectionMask, lerp(kRed, kGreen, LowFrequencyRejection), Color);

	return GrayScaleColor(ResurrectedColor) * Color;
}

tsr_half3 DisplaySpatialAntiAliasingMask(FContext Ctx)
{
	bool bSpatialAntiAlias = AntiAliasMaskTexture[Ctx.InputPixelPos] != tsr_ushort(0x0);
	if (!bCanSpatialAntiAlias)
	{
		return SampleTSROutputGray(Ctx) * kPink;
	}
	return SampleTSROutputGray(Ctx) * select(bSpatialAntiAlias, kGreen, kWhite);
}

tsr_half3 DisplayAntiFlickering(FContext Ctx)
{
	if (FlickeringFramePeriod == 0.0)
	{
		return SampleTSROutputGray(Ctx) * kPink;
	}

	const tsr_half BlendFinal = tsr_half(0.05);
	const tsr_half MoireEncodingError = tsr_half(rcp(127.0));
	const tsr_half MaxPrevTotalVariationCount = tsr_half(20.0);
	
	// Compute the threshold at which the variation count should quick in
	const tsr_half TotalVariationCountThreshold = tsr_half(1.0 / (1.0 - pow(1.0 - BlendFinal, tsr_half(FlickeringFramePeriod))));

	// Texture coordinate
	float2 MoireHistoryUV = ApplyScreenTransform(ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, Ctx.ScreenPos), ScreenPosToMoireHistoryUV);
	MoireHistoryUV = fastClamp(MoireHistoryUV, MoireHistoryUVBilinearMin, MoireHistoryUVBilinearMax);

	// Texture fetches
	tsr_half3 OriginalOpaqueInput = InputTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0);
	tsr_half  MoireInput = InputMoireLumaTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0);
	tsr_half4 OriginalTranslucencyInput = InputSceneTranslucencyTexture.SampleLevel(GlobalPointClampedSampler, Ctx.InputUV, 0);
	tsr_half4 MoireHistory = MoireHistoryTexture.SampleLevel(GlobalPointClampedSampler, MoireHistoryUV, 0);
	tsr_half DisableAntiFlickering = tsr_half(1.0) - tsr_half(IsMovingMaskTexture[Ctx.InputPixelPos]) * rcp(tsr_half(255.0));

	// Unpack history
	tsr_half FlickeringHistory   = GCSToSMCS(MoireHistory[0]);
	tsr_half Gradient            = MoireHistory[1] * tsr_half(255.0 / 127.0) - tsr_half(1.0);
	tsr_half TotalVariation      = MoireHistory[2] * (tsr_half(1.0) - BlendFinal);
	tsr_half TotalVariationCount = MoireHistory[3] * MaxPrevTotalVariationCount * (tsr_half(1.0) - BlendFinal);
	
	tsr_half InputLumaModification;
	{
		const tsr_half InputLumaEncodingError = tsr_half(0.5 / 255.0);
		
		tsr_half3 OriginalInput = OriginalOpaqueInput * OriginalTranslucencyInput.a + OriginalTranslucencyInput.rgb;
		
		OriginalInput = LinearToSMCS(OriginalInput * tsr_half(ExposureOffsetFactor));
		OriginalOpaqueInput = LinearToSMCS(OriginalOpaqueInput * tsr_half(ExposureOffsetFactor));

		tsr_half SceneColorLuma = dot(OriginalOpaqueInput.rgb, kMoireSMCSWeights);
		tsr_half TranslucencyChange = dot(abs(OriginalOpaqueInput.rgb - OriginalInput.rgb), kMoireSMCSWeights);
		
		tsr_half InputLuma = GCSToSMCS(MoireInput);

		InputLumaModification = tsr_half(0.0);
		InputLumaModification = max(InputLumaModification, abs(InputLuma - SceneColorLuma) - InputLumaEncodingError);
		InputLumaModification = max(InputLumaModification, TranslucencyChange);
		InputLumaModification = max(InputLumaModification, tsr_half(1.0) - OriginalTranslucencyInput[3]);
	}

	tsr_half InputLumaModificationPercent = saturate(InputLumaModification * tsr_half(16.0));

	// Compute the final luminance 
	#if 1
		tsr_half LocalCountFadeIn = saturate(TotalVariationCount / TotalVariationCountThreshold - tsr_half(0.5));
	#else
		tsr_half LocalCountFadeIn = saturate(TotalVariationCount - TotalVariationCountThreshold);
	#endif
	tsr_half LocalMoireError = (abs(TotalVariation * SafeRcp(TotalVariationCount)) + TotalVariationCount * MoireEncodingError);
	
	tsr_half3 Color;
	Color = lerp(kGreen, kYellow, saturate(LocalMoireError * 16.0));
	Color = lerp(Color,  kRed,    saturate(LocalMoireError * 16.0 - 1.0));
	Color = lerp(kWhite, Color,   LocalCountFadeIn);
	Color = lerp(kPink,  Color,   min(DisableAntiFlickering, tsr_half(1.0) - InputLumaModificationPercent));
	Color = select(bCameraCut == 0, Color, kPink);

	return SampleTSROutputGray(Ctx) * Color;
}


//------------------------------------------------------- REPROJECTION FIELD

tsr_half3 DisplayReprojectionFieldSummary(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half JacobianVisibilityMultiplier = tsr_half(64.0);
	//tsr_half JacobianVisibilityMultiplier = tsr_half(select(Reprojection.bAntiAliasReprojectionField, 2, 1)) * tsr_half(64.0);

	tsr_half MaxJacobianX = max(abs(Reprojection.Jacobian[0][0]), abs(Reprojection.Jacobian[0][1]));
	tsr_half MaxJacobianY = max(abs(Reprojection.Jacobian[1][0]), abs(Reprojection.Jacobian[1][1]));

	tsr_half3 Color;
	Color = kWhite;
	Color.g = lerp(Color.g, tsr_half(1.0), saturate(MaxJacobianX * JacobianVisibilityMultiplier));
	Color.b = lerp(Color.b, tsr_half(1.0), saturate(MaxJacobianY * JacobianVisibilityMultiplier));
	Color = select(Reprojection.bIsOffScreen, kPink, Color);
	//Color = select(Reprojection.bAntiAliasReprojectionField, kGreen, Color);
	Color = select(!bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionFieldOffset(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half3 Color = kWhite;
	Color.gb = tsr_half2((float2(Reprojection.Offset) * 0.5 + 0.5) * 0.67 + 0.33);
	Color = select(all(Reprojection.Offset == tsr_short(0)), select(Reprojection.bIsOffScreen, kPink, kWhite), Color);
	Color = select(!bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionOffsetCoverage(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half ReprojectionBoundaryCoverage = ComputeReprojectionBoundaryCoverage(Reprojection.Boundary);

	tsr_half3 Color;
	Color = lerp(kRed, kGreen, ReprojectionBoundaryCoverage);
	Color = select(Reprojection.bIsFullDilate, kCyan, Color);
	Color = select(all(Reprojection.Offset == tsr_short(0)), select(Reprojection.bIsOffScreen, kPink, kWhite), Color);
	Color = select(!bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionNullJacobian(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half MaxJacobianX = max(abs(Reprojection.Jacobian[0][0]), abs(Reprojection.Jacobian[0][1]));
	tsr_half MaxJacobianY = max(abs(Reprojection.Jacobian[1][0]), abs(Reprojection.Jacobian[1][1]));
	tsr_half MaxJacobian = max(MaxJacobianX, MaxJacobianY);
	
	bool bIsNullVector = all(abs(Reprojection.Vector) < (0.5 / 65535.0));
	bool bIsNullJacobian = MaxJacobian < pow(tsr_half(1u << kVelocityJacobianBits), tsr_half(-2.0));

	tsr_half3 Color = select(bIsNullJacobian && !bIsNullVector, kOrange, kWhite);
	Color = select(Reprojection.bIsOffScreen || !bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionClampedJacobian(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half MaxJacobianX = max(abs(Reprojection.Jacobian[0][0]), abs(Reprojection.Jacobian[0][1]));
	tsr_half MaxJacobianY = max(abs(Reprojection.Jacobian[1][0]), abs(Reprojection.Jacobian[1][1]));
	tsr_half MaxJacobian = max(MaxJacobianX, MaxJacobianY);

	tsr_half3 Color;
	Color = select(Reprojection.bIsOffScreen, kPink, kWhite);
	Color = select(MaxJacobian > kVelocityJacobianRange - rcp(tsr_half(1u << kVelocityJacobianBits)), kRed, Color);
	Color = select(!bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionFieldDilatedJacobian(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half ReprojectionUpscaleFactor = ComputeReprojectionUpscaleFactorFromJacobian(Reprojection.Jacobian);

	tsr_half3 Color;
	Color = kWhite;
	Color = lerp(Color,   kRed, saturate( log2(ReprojectionUpscaleFactor)));
	Color = lerp(Color, kGreen, saturate(-log2(ReprojectionUpscaleFactor)));
	Color = select(Reprojection.bIsOffScreen || !bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}

tsr_half3 DisplayReprojectionFieldAA(FContext Ctx)
{
	FReprojectionFieldSample Reprojection = SampleReprojectionField(Ctx);
	
	tsr_half ReprojectionUpscaleFactor = ComputeReprojectionUpscaleFactorFromJacobian(Reprojection.Jacobian);

	tsr_half3 Color;
	Color = select(Reprojection.bIsOffScreen, kPink, kWhite);
	Color = select(Reprojection.bAntiAliasReprojectionField, kGreen, Color);
	Color = select(!bReprojectionField, kPink, Color);
	return SampleTSROutputGray(Ctx) * Color;
}


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE * TILE_SIZE, 1, 1)]
void VisualizeCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	uint GroupWaveIndex = GetGroupWaveIndex(GroupThreadIndex, /* GroupSize = */ TILE_SIZE * TILE_SIZE);

	Debug = 0.0;
	
	FContext Ctx;
	Ctx.OutputPixelPos = (
		tsr_short2(OutputViewRectMin) +
		tsr_short2(GroupId) * tsr_short2(TILE_SIZE, TILE_SIZE) +
		Map8x8Tile2x2Lane(GroupThreadIndex));
	Ctx.ScreenPos = ApplyScreenTransform(float2(Ctx.OutputPixelPos), OutputPixelPosToScreenPos);
	Ctx.InputUV = ApplyScreenTransform(ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, Ctx.ScreenPos), ScreenPosToInputUV);
	Ctx.InputUV = fastClamp(Ctx.InputUV, InputInfo_UVViewportBilinearMin, InputInfo_UVViewportBilinearMax);
	Ctx.InputPixelPos = tsr_short2(ApplyScreenTransform(ApplyLensDistortionOnScreenPos(UndistortingDisplacementTexture, Ctx.ScreenPos), ScreenPosToInputPixelPos));
	Ctx.InputPixelPos = fastClamp(Ctx.InputPixelPos, tsr_short2(InputInfo_ViewportMin), tsr_short2(InputInfo_ViewportMax - 1));

	tsr_half3 OutputColor = tsr_half(0.0);

	BRANCH
	if (VisualizeId == 0)
	{
		OutputColor = DisplayHistorySampleCount(Ctx);
	}
	else if (VisualizeId == 1)
	{
		OutputColor = DisplayParallaxDisocclusionMask(Ctx);
	}
	else if (VisualizeId == 2)
	{
		OutputColor = DisplayHistoryRejection(Ctx);
	}
	else if (VisualizeId == 3)
	{
		OutputColor = DisplayHistoryClamp(Ctx);
	}
	else if (VisualizeId == 4)
	{
		OutputColor = DisplayResurrectionMask(Ctx);
	}
	else if (VisualizeId == 5)
	{
		OutputColor = DisplayResurrectedColor(Ctx);
	}
	else if (VisualizeId == 6)
	{
		OutputColor = DisplaySpatialAntiAliasingMask(Ctx);
	}
	else if (VisualizeId == 7)
	{
		OutputColor = DisplayAntiFlickering(Ctx);
	}
	else if (VisualizeId == 8)
	{
		OutputColor = DisplayReprojectionFieldSummary(Ctx);
	}
	else if (VisualizeId == 9)
	{
		OutputColor = DisplayReprojectionFieldOffset(Ctx);
	}
	else if (VisualizeId == 10)
	{
		OutputColor = DisplayReprojectionOffsetCoverage(Ctx);
	}
	else if (VisualizeId == 11)
	{
		OutputColor = DisplayReprojectionFieldAA(Ctx);
	}
	else if (VisualizeId == 12)
	{
		OutputColor = DisplayReprojectionNullJacobian(Ctx);
	}
	else if (VisualizeId == 13)
	{
		OutputColor = DisplayReprojectionClampedJacobian(Ctx);
	}
	else if (VisualizeId == 14)
	{
		OutputColor = DisplayReprojectionFieldDilatedJacobian(Ctx);
	}
	else if (VisualizeId == 15)
	{
		OutputColor = DisplayThinGeometry(Ctx);
	}

	{
		tsr_short2 OutputPixelPos = InvalidateOutputPixelPos(Ctx.OutputPixelPos, OutputViewRectMax);
		
		Output[OutputPixelPos] = tsr_half4(OutputColor, 0.0);
		
		#if DEBUG_OUTPUT
		{
			DebugOutput[tsr_short3(OutputPixelPos, 0)] = Debug;
		}
		#endif
	}
}
