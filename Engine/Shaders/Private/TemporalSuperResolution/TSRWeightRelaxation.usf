// Copyright Epic Games, Inc. All Rights Reserved.



#define CONFIG_TRANSLUCENCY_COVERAGE_BIT (DIM_REJECT_TRANSLUCENCY_DETECTION)

//------------------------------------------------------- INCLUDES
#include "TSRColorSpace.ush"
//#include "TSRDepthVelocityAnalysis.ush"
#include "TSRThinGeometryCommon.ush"

//------------------------------------------------------- CONFIG

#define DEBUG_ARRAY_SIZE 8

#define DEBUG_DISABLE_PADDING 0

#define DEBUG_DIGESTABLE 0

//------------------------------------------------------- CONSTANTS

#if CONFIG_LDS_DESPILL
static const uint kDespillInput = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 0;
static const uint kDespillHistory = LDS_DESPILL_THREAD_COUNT * LDS_DESPILL_OFFSET * 1;
#endif

//------------------------------------------------------- PARAMETERS

int TileOverscan;
int ThinGeometryTextureIndex;
float ErrorMultiplier;
float MaxRelaxationWeight;
float TranslucencyRejectionThreshold;
int bIntensityLineDetectionEnabled;

Texture2D<tsr_half>  CurrentCoverageTexture;
Texture2D<tsr_half>  InputMoireLumaTexture;
Texture2D<tsr_halfC> InputTexture;
Texture2D<tsr_half4> InputSceneTranslucencyTexture;

RWTexture2DArray<tsr_ushort> R8Output;
RWTexture2DArray<tsr_half> HistoryCoverageOutput;

//------------------------------------------------------- GLOBALS
static uint2 GGroupId = 0;

//------------------------------------------------------- DEBUG

static tsr_tensor_half4 Debug[DEBUG_ARRAY_SIZE];

//------------------------------------------------------- INPUT FETCHES

tsr_short2 ComputeElementPixelPos(const uint ElementIndex)
{
	tsr_short2 ClampedFetchPixelPos = ComputeElementInputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);

	return ClampedFetchPixelPos;
}

tsr_tensor_short2 ComputePixelPos()
{
	tsr_tensor_short2 PixelPos;

	UNROLL_N(SIMD_SIZE)
		for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			PixelPos.SetElement(ElementIndex, ComputeElementPixelPos(ElementIndex));
		}

	return PixelPos;
}

tsr_short2 ComputeElementOutputPixelPos(const uint ElementIndex)
{
	return ComputeElementOutputPixelPos(
		/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
		TileOverscan,
		InputInfo_ViewportMin,
		InputInfo_ViewportMax,
		GGroupId,
		GGroupThreadIndex,
		ElementIndex);
}

CALL_SITE_DEBUGLOC
tsr_half ComputeRGBLuma(tsr_half3 Input)
{
	// aligns to the logic in MeasureFlickeringLuma.usf so taht we can compare the 
	// difference between LumaBeforeComposition that comes from that pass.
	tsr_half3 LinearColor = Input;
	tsr_half3 SMColor = LinearToSMCS(LinearColor);
	tsr_half  SMLuma = dot(SMColor, kMoireSMCSWeights);
	tsr_half  GLuma = SMCSToGCS(SMLuma.rrr).r;

	return GLuma;
}

CALL_SITE_DEBUGLOC
tsr_tensor_half ComputeRGBLuma(tsr_tensor_half3 Input)
{
	tsr_tensor_half Luma;
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		// aligns to the logic in MeasureFlickeringLuma.usf so taht we can compare the 
		// difference between LumaBeforeComposition that comes from that pass.
		tsr_half3 LinearColor = Input.GetElement(ElementIndex);
		Luma.SetElement(ElementIndex, ComputeRGBLuma(LinearColor));
	}

	return Luma;
}

void FetchInput(
	tsr_tensor_short2 PixelPos,
	out tsr_tensor_half InputLuma,
	out tsr_tensor_half2 Translucency,
	out tsr_tensor_half HistoryRelaxationWeight,
	out tsr_tensor_bool IsClusterHoleRegion
)
{
	tsr_tensor_ushort ThinGeometryCache;
	
	UNROLL_N(SIMD_SIZE)
	for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
	{
		tsr_short2 ClampedFetchPixelPos = PixelPos.GetElement(ElementIndex);
		InputLuma.SetElement(ElementIndex,ComputeRGBLuma(InputTexture[ClampedFetchPixelPos].xyz));
		tsr_half4 FullTranslucency = InputSceneTranslucencyTexture[ClampedFetchPixelPos];
		Translucency.SetElement(ElementIndex, tsr_half2(ComputeRGBLuma(FullTranslucency.rgb),FullTranslucency.a));
		ThinGeometryCache.SetElement(ElementIndex, tsr_ushort(R8Output[int3(ClampedFetchPixelPos, ThinGeometryTextureIndex)]));
	}

	DecodeThinGeometry(ThinGeometryCache, HistoryRelaxationWeight, IsClusterHoleRegion);	
}

//------------------------------------------------------- ENTRY POINT

#if COMPILER_SUPPORTS_WAVE_SIZE && DIM_WAVE_SIZE > 0
WAVESIZE(DIM_WAVE_SIZE)
#endif
[numthreads(LANE_COUNT * WAVE_COUNT, 1, 1)]
void WeightRelaxationCS(
	uint2 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	GGroupId = GroupId;
	GGroupThreadIndex = GroupThreadIndex;

#if DEBUG_OUTPUT
	{
		UNROLL_N(DEBUG_ARRAY_SIZE)
			for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
			{
				Debug[DebugId].SetAllElements(0.0);
			}
	}
#endif
	tsr_tensor_short2 PixelPos = ComputePixelPos();
	tsr_tensor_half InputLuma;
	tsr_tensor_half2 Translucency;
	tsr_tensor_half HistoryRelaxationWeight;
	tsr_tensor_bool IsPartialCoverageRegion;

	FetchInput(
		PixelPos,
 		/*out */ InputLuma,
		/*out */ Translucency,
		/*out */ HistoryRelaxationWeight,
		/*out */ IsPartialCoverageRegion
	);

	// If translucency has data turn down the weights aggresively. if alpha < 0.95
	tsr_tensor_half TranslucencyRatio = Translucency[0] * rcp(Translucency[0] + InputLuma * Translucency[1] + tsr_tensor_half::Const(0.0005));

	// Weights decline starts around TranslucencyRatio = 0.05, and declines fast to 0 when Translucency ratio goes to 0.6. 
	tsr_tensor_half RatioToLerpWeight = max(tsr_tensor_half::Const(0.6)-TranslucencyRatio,tsr_tensor_half::Const(0));
	RatioToLerpWeight = min(RatioToLerpWeight * RatioToLerpWeight * tsr_tensor_half::Const(3.33), tsr_tensor_half::Const(1));
	RatioToLerpWeight = select(or(IsPartialCoverageRegion,TranslucencyRatio > tsr_tensor_half::Const(0.6)), RatioToLerpWeight, tsr_tensor_half::Const(1));
	
	// Write only when the weights has been modified.
	bool bRelaxationWeightsModified = AnyComponent(and(HistoryRelaxationWeight > tsr_tensor_half::Const(tsr_half(0)), RatioToLerpWeight < tsr_tensor_half::Const(tsr_half(1.0))));
	
	BRANCH
	if (bRelaxationWeightsModified)
	{
		HistoryRelaxationWeight = lerp(tsr_tensor_half::Const(0), HistoryRelaxationWeight, RatioToLerpWeight); 
		tsr_tensor_ushort EncodedThinGeometry = EncodeThinGeometry(HistoryRelaxationWeight,IsPartialCoverageRegion);
		// Output
		UNROLL_N(SIMD_SIZE)
			for(uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
			{
				tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
				R8Output[int3(OutputPixelPos, ThinGeometryTextureIndex)] = EncodedThinGeometry.GetElement(ElementIndex);
			}
	}
	
	// Write translucency bits only when enabled.
	#if CONFIG_TRANSLUCENCY_COVERAGE_BIT
	tsr_tensor_bool HaveMainTranslucencyContribution = TranslucencyRatio > tsr_tensor_half::Const(tsr_half(TranslucencyRejectionThreshold));
	bool bHaveMainTranslucencyContribution = AnyComponent(HaveMainTranslucencyContribution);

	BRANCH
	if (bHaveMainTranslucencyContribution)
	{
		// Output
		UNROLL_N(SIMD_SIZE)
		for(uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
		{
			tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(ElementIndex);
			// Reject more aggressively.
			HistoryCoverageOutput[uint3(OutputPixelPos, 0)] = 
			OverwriteCoverageWithTranslucencyBit(tsr_half(0), HaveMainTranslucencyContribution.GetElement(ElementIndex), bIntensityLineDetectionEnabled);
		}
	}
	#endif

#if DEBUG_OUTPUT
	{
		// Mark the top left corner of the tile for debugging purposes.
#if DEBUG_DISABLE_PADDING == 2
		if (GGroupThreadIndex == 0)
		{
			UNROLL_N(DEBUG_ARRAY_SIZE)
				for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
				{
					Debug[DebugId].SetElement(/* ElementIndex = */ 0, 1.0);
				}
		}
#endif // DEBUG_DISABLE_PADDING == 2

		UNROLL_N(SIMD_SIZE)
			for (uint ElementIndex = 0; ElementIndex < SIMD_SIZE; ElementIndex++)
			{
				const tsr_short2 OutputPixelPos = ComputeElementOutputPixelPos(
					/* LaneStride = */ uint2(LANE_STRIDE_X, LANE_STRIDE_Y),
					TileOverscan,
					InputInfo_ViewportMin,
					InputInfo_ViewportMax,
					GGroupId,
					GGroupThreadIndex,
					ElementIndex,
					/* OutputDataOffset = */ int2(0, 0),
					/* OutputResDivisor = */ 1,
					/* bDebugDisablePadding = */ DEBUG_DISABLE_PADDING != 0);

				UNROLL_N(DEBUG_ARRAY_SIZE)
					for (uint DebugId = 0; DebugId < DEBUG_ARRAY_SIZE; DebugId++)
					{
						DebugOutput[tsr_short3(OutputPixelPos, DebugId)] = float4(Debug[DebugId].GetElement(ElementIndex));
					}
			}
	}
#endif // DEBUG_OUTPUT

}
