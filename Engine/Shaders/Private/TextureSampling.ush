// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextureSampling.usf: Hold helper to do custom texture sampling.
=============================================================================*/

#pragma once


/** Returns the exact UV coordinate of the center of the closest pixel. */
float2 SnapToClosestPixelCenterUVEx(float2 SampleUV, float2 TextureSize, float2 TextureInvSize, float MipLevelPow2, float InvMipLevelPow2)
{
	return (TextureInvSize * InvMipLevelPow2) * (floor((TextureSize * MipLevelPow2) * SampleUV) + 0.5);
}

float2 SnapToClosestPixelCenterUV(float2 SampleUV, float2 TextureSize, float2 TextureInvSize, const uint MipLevel = 0)
{
	return SnapToClosestPixelCenterUVEx(SampleUV, TextureSize, TextureInvSize, float(1u << MipLevel), rcp(float(1u << MipLevel)));
}


/** Pixel offsets to use for manual bilinear filtering. */
static const uint2 BilinearSamplingOffsets2x2[4] =
{
	int2(0,  0),
	int2(1,  0),
	int2(0,  1),
	int2(1,  1),
};


/** Holds all information to do bilinear sample manually. */
struct FBilinearSampleInfos
{
	// The texture's inverse size.
	float2 TextureInvSize;

	// Pixel coordinate of the top left sample.
	float2 TopLeftPixelCoord;

	// Bilinear interpolation to perform on X and Y axis.
	float2 BilinearInterp;
};


/** Generate all information necessary for doing bilinear texture sampling manually. */
FBilinearSampleInfos GetBilinearSampleLevelInfosEx(float2 SampleUV, float2 TextureSize, float2 TextureInvSize, float MipLevelPow2, float InvMipLevelPow2)
{
	FBilinearSampleInfos Infos;

	float2 HistoryBufferPixelCoord = SampleUV * TextureSize * InvMipLevelPow2;

	Infos.TextureInvSize = TextureInvSize * MipLevelPow2;
	Infos.TopLeftPixelCoord = floor(HistoryBufferPixelCoord - 0.5);
	Infos.BilinearInterp = frac(HistoryBufferPixelCoord - 0.5);
	return Infos;
}

FBilinearSampleInfos GetBilinearSampleLevelInfos(float2 SampleUV, float2 TextureSize, float2 TextureInvSize, const uint MipLevel = 0)
{
	return GetBilinearSampleLevelInfosEx(SampleUV, TextureSize, TextureInvSize, float(1u << MipLevel), rcp(float(1u << MipLevel)));
}

/** Return the weight of a sample <SampleId>. */
float GetSampleWeight(FBilinearSampleInfos SampleInfos, const uint SampleId)
{
	float BilinearSampleWeights[4] = {
		(1 - SampleInfos.BilinearInterp.x) * (1 - SampleInfos.BilinearInterp.y),
		(SampleInfos.BilinearInterp.x) * (1 - SampleInfos.BilinearInterp.y),
		(1 - SampleInfos.BilinearInterp.x) * (SampleInfos.BilinearInterp.y),
		(SampleInfos.BilinearInterp.x) * (SampleInfos.BilinearInterp.y),
	};
	return BilinearSampleWeights[SampleId];
}

/** Return the uv of the sample <SampleId>. */
float2 GetSampleUV(FBilinearSampleInfos SampleInfos, const uint SampleId)
{
	return (SampleInfos.TopLeftPixelCoord + (BilinearSamplingOffsets2x2[SampleId] + 0.5)) * SampleInfos.TextureInvSize;
}

/** Return the pixel coordinate of sample <SampleId>. */
uint2 GetSamplePixelCoord(FBilinearSampleInfos SampleInfos, const uint SampleId)
{
	return uint2(int2(SampleInfos.TopLeftPixelCoord)) + BilinearSamplingOffsets2x2[SampleId];
}

/** Bilinearly interpolates the four given samples. */
float GetBilinearInterpolation(FBilinearSampleInfos SampleInfos, float4 Samples)
{
	float4 BilinearSampleWeights;
	BilinearSampleWeights[0] = (1 - SampleInfos.BilinearInterp.x) * (1 - SampleInfos.BilinearInterp.y);
	BilinearSampleWeights[1] = (SampleInfos.BilinearInterp.x) * (1 - SampleInfos.BilinearInterp.y);
	BilinearSampleWeights[2] = (1 - SampleInfos.BilinearInterp.x) * (SampleInfos.BilinearInterp.y);
	BilinearSampleWeights[3] = (SampleInfos.BilinearInterp.x) * (SampleInfos.BilinearInterp.y);

	return dot(Samples, BilinearSampleWeights);
}

#define BICUBIC_CATMULL_ROM_SAMPLES 5

struct FCatmullRomSamples
{
	// Constant number of samples (BICUBIC_CATMULL_ROM_SAMPLES)
	uint Count;

	// Constant sign of the UV direction from main UV sampling location.
	int2 UVDir[BICUBIC_CATMULL_ROM_SAMPLES];

	// Bilinear sampling UV coordinates of the samples
	float2 UV[BICUBIC_CATMULL_ROM_SAMPLES];

	// Weights of the samples
	half Weight[BICUBIC_CATMULL_ROM_SAMPLES];

	// Final multiplier (it is faster to multiply 3 RGB values than reweights the 5 weights)
	float FinalMultiplier;
};

FCatmullRomSamples GetBicubic2DCatmullRomSamples(float2 UV, float2 Size, in float2 InvSize)
{
	float2 PixelCoord = UV * Size;

	float2 iuv = floor(PixelCoord - 0.5) + 1.0;
	half2 f = half2(PixelCoord - iuv);
	half2 f2 = f * f;
	half2 f_2 = f + f;

	half2 uv = (half(1.25) - f2) * f + half(0.5);
	half2 t0 = (half(0.25) * f2 - half(0.0625)) * (half(1.125) - half(0.5) * f2.yx);
	half2 w0 = t0 - f_2 * t0;
	half2 w3 = t0 + f_2 * t0;

	float2 cuv = (iuv - 0.5) * InvSize;
	uv *= half2(InvSize);

	FCatmullRomSamples Samples;
	Samples.Count = BICUBIC_CATMULL_ROM_SAMPLES;

	Samples.UV[0] = cuv + half2(-InvSize.x, uv.y);
	Samples.UV[1] = cuv + half2(half(2.0) * InvSize.x, uv.y);
	Samples.UV[2] = cuv + uv;
	Samples.UV[3] = cuv + half2(uv.x, -InvSize.y);
	Samples.UV[4] = cuv + half2(uv.x, half(2.0) * InvSize.y);
	
	Samples.Weight[0] = half(w0.x);
	Samples.Weight[1] = half(w3.x);
	Samples.Weight[2] = half(half(1.0) - w0.x - w3.x - w0.y - w3.y);
	Samples.Weight[3] = half(w0.y);
	Samples.Weight[4] = half(w3.y);

	Samples.UVDir[0] = int2(-1,  0);
	Samples.UVDir[1] = int2( 1,  0);
	Samples.UVDir[2] = int2( 0,  0);
	Samples.UVDir[3] = int2( 0, -1);
	Samples.UVDir[4] = int2( 0,  1);
	
	Samples.FinalMultiplier = 1.0;
	return Samples;
}

MaterialFloat4 Texture2DSampleBicubic(Texture2D Tex, SamplerState Sampler, float2 UV, float2 Size, in float2 InvSize)
{
	FCatmullRomSamples Samples = GetBicubic2DCatmullRomSamples(UV, Size, InvSize);

	float4 OutColor = 0;
	for (uint i = 0; i < Samples.Count; i++)
	{
		OutColor += Tex.SampleLevel(Sampler, Samples.UV[i], 0) * Samples.Weight[i];
	}
	OutColor *= Samples.FinalMultiplier;

	return OutColor;
}
