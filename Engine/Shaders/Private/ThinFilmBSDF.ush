// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/////////////////////////////////////////////////////////////////////////////////////////////////
// Iridescence

// Depolarization functions for natural light
float ThinFilmDepol(float2 polV) { return 0.5 * (polV.x + polV.y); }
float3 ThinFilmDepolColor(float3 colS, float3 colP) { return 0.5 * (colS + colP); }

// Fresnel equations for dielectric/dielectric interfaces.
void ThinFilmFresnelDielectric(
	in float ct1,
	in float n1,
	in float n2,
	out float2 R,
	out float2 phi)
{
	float st1 = (1 - ct1 * ct1); // Sinus theta1 'squared'
	float nr = n1 / n2;

	if (Pow2(nr) * st1 > 1)  // Total reflection
	{
		R = float2(1, 1);
		phi = 2.0 * atan(float2(
			-Pow2(nr) * sqrt(st1 - 1.0 / Pow2(nr)) / ct1,
			-sqrt(st1 - 1.0 / Pow2(nr)) / ct1));
	}
	else    // Transmission & Reflection
	{

		float ct2 = sqrt(1 - Pow2(nr) * st1);
		float2 r = float2(
			(n2 * ct1 - n1 * ct2) / (n2 * ct1 + n1 * ct2),
			(n1 * ct1 - n2 * ct2) / (n1 * ct1 + n2 * ct2));
		phi.x = (r.x < 0.0) ? PI : 0.0;
		phi.y = (r.y < 0.0) ? PI : 0.0;
		R = Pow2(r);
	}
}

// Fresnel equations for dielectric/conductor interfaces.
void ThinFilmFresnelConductor(
	in float ct1,
	in float n1,
	in float n2,
	in float k,
	out float2 R,
	out float2 phi)
{
	if (k == 0)
	{ // use dielectric formula to avoid numerical issues
		ThinFilmFresnelDielectric(ct1, n1, n2, R, phi);
	}
	else
	{
		float A = Pow2(n2) * (1 - Pow2(k)) - Pow2(n1) * (1 - Pow2(ct1));
		float B = sqrt(Pow2(A) + Pow2(2 * Pow2(n2) * k));
		float U = sqrt((A + B) / 2.0);
		float V = sqrt((B - A) / 2.0);

		R.y = (Pow2(n1 * ct1 - U) + Pow2(V)) / (Pow2(n1 * ct1 + U) + Pow2(V));
		phi.y = atan2(2 * n1 * V * ct1, Pow2(U) + Pow2(V) - Pow2(n1 * ct1)) + PI;

		R.x = (Pow2(Pow2(n2) * (1 - Pow2(k)) * ct1 - n1 * U) + Pow2(2 * Pow2(n2) * k * ct1 - n1 * V)) / (Pow2(Pow2(n2) * (1 - Pow2(k)) * ct1 + n1 * U) + Pow2(2 * Pow2(n2) * k * ct1 + n1 * V));
		phi.x = atan2(2 * n1 * Pow2(n2) * ct1 * (2 * k * U - (1 - Pow2(k)) * V), Pow2(Pow2(n2) * (1 + Pow2(k)) * ct1) - Pow2(n1) * (Pow2(U) + Pow2(V)));
	}
}

// Evaluation XYZ sensitivity curves in Fourier space
float3 ThinFilmEvalSensitivity(float opd, float shift)
{
	// Use Gaussian fits, given by 3 parameters: val, pos and var
	float phase = 2 * PI * opd * 1.0e-6;
	float3 val = float3(5.4856e-13, 4.4201e-13, 5.2481e-13);
	float3 pos = float3(1.6810e+06, 1.7953e+06, 2.2084e+06);
	float3 var = float3(4.3278e+09, 9.3046e+09, 6.6121e+09);
	float3 xyz = val * sqrt(2 * PI * var) * cos(pos * phase + shift) * exp(-var * phase * phase);
	xyz.x += 9.7470e-14 * sqrt(2 * PI * 4.5282e+09) * cos(2.2399e+06 * phase + shift) * exp(-4.5282e+09 * phase * phase);
	return xyz / 1.0685e-7;
	// As pointed by the suplemental material the oupout is not in the correct space, we need to tranfser 
}


// [Gulbrandsen, "Artist Friendly Metallic Fresnel"]
// Compute complex index of refraction N = n + ik from F0 and edge tint.
void ComputeComplexIORFromF0AndEdgeTint(in float3 Reflectivity, in float3 EdgeTint, inout float3 n, inout float3 k)
{
	const float3 g = EdgeTint;
	const float3 r = clamp(Reflectivity, 0.0, 0.999);
	const float3 SqrtR = sqrt(r);

	n = g * (1 - r) / (1 + r) + (1 - g) * (1 + SqrtR) / (1 - SqrtR);

	const float3 k2 = ((n + 1) * (n + 1) * r - (n - 1) * (n - 1)) / (1 - r);
	k = sqrt(k2);
}

// Fresnel term for iridescent microfacet BRDF model 
// This is a reference implementation adapted from the original paper's source code
float3 F_ThinFilmRef(float NoV, float NoL, float VoH, float3 F0, float3 F90, float ThinFilmIOR, float ThinFilmTickness)
{
	// Translate F0/F90 to eta/kappa form
	float3 n3 = 1;
	float3 k3 = 0;
	ComputeComplexIORFromF0AndEdgeTint(F0, F90, n3, k3);

	// Thin film is limited (for now) to the top most layer (i.e., the upper layer need to be air)
	//					|
	//	  n1=1 / k1=0	| Air
	// ----------------- 
	//		n2 / k2=0	| Thin film
	// ----------------- 
	//		n3 / k3		| Regular material
	//					|
	//
	//		        Min		Max		Default
	// float Dinc	0.0		10.0	0.5
	// float eta2	1.0		5.0		2.0
	// float eta3	1.0		5.0		3.0
	// float kappa3	0.0		5.0		0.0
	// 
	// n1 is fixed to air (~1.f)
	float3 Avg = 1.f / 3.f;
	float eta2	 = clamp(ThinFilmIOR, 1.f, 3.f);
	float eta3	 = clamp(dot(Avg,n3), 1.f, 3.f);
	float kappa3 = clamp(dot(Avg,k3), 1.f, 5.f);
	float Dinc   = ThinFilmTickness * 10; // ThinFilmTickness is in micrometer. Range is 0 - 10um i.e, 0 - 10 000nm

	// Force eta_2 -> 1.0 when Dinc -> 0.0
	const float eta_2 = lerp(1.0, eta2, smoothstep(0.0, 0.03, Dinc));

	// Compute dot products
	const bool bIsValid = NoL > 0 && NoV > 0;

	float3 I = 0;
	if (bIsValid)
	{
		float cosTheta1 = VoH;
		float cosTheta2 = sqrt(1.0 - Pow2(1.0 / eta_2) * (1 - Pow2(cosTheta1)));

		// First interface
		float2 R12, phi12;
		ThinFilmFresnelDielectric(cosTheta1, 1.0, eta_2, R12, phi12);
		float2 R21 = R12;
		float2 T121 = float2(1, 1) - R12;
		float2 phi21 = float2(PI, PI) - phi12;

		// Second interface
		float2 R23, phi23;
		ThinFilmFresnelConductor(cosTheta2, eta_2, eta3, kappa3, R23, phi23);

		// Phase shift
		float OPD = Dinc * cosTheta2;
		float2 phi2 = phi21 + phi23;

		// Compound terms
		float2 R123 = R12 * R23;
		float2 r123 = sqrt(R123);
		float2 Rs = Pow2(T121) * R23 / (1 - R123);

		// Reflectance term for m=0 (DC term amplitude)
		float2 C0 = R12 + Rs;
		float3 S0 = ThinFilmEvalSensitivity(0.0, 0.0);
		I += ThinFilmDepol(C0) * S0;

		// Reflectance term for m>0 (pairs of diracs)
		float2 Cm = Rs - T121;
		for (int m = 1; m <= 3; ++m)
		{
			Cm *= r123;
			float3 SmS = 2.0 * ThinFilmEvalSensitivity(m * OPD, m * phi2.x);
			float3 SmP = 2.0 * ThinFilmEvalSensitivity(m * OPD, m * phi2.y);
			I += ThinFilmDepolColor(Cm.x * SmS, Cm.y * SmP);
		}

		// Convert back to RGB reflectance
		// XYZ to CIE 1931 RGB color space (using neutral E illuminant)
		const float3x3 ThinFilm_XYZ_TO_RGB = float3x3(2.3706743, -0.5138850, 0.0052982, -0.9000405, 1.4253036, -0.0146949, -0.4706338, 0.0885814, 1.0093968);
		I = clamp(mul(I, ThinFilm_XYZ_TO_RGB), float3(0, 0, 0), float3(1, 1, 1));
	}
	return I;
}

// Fresnel term for iridescent microfacet BRDF model 
// Simplified version which relies on Schlick's Fresnel and de facto does not take into 
// account Fresnel phase shift & polarization.
float3 F_ThinFilm(float NoV, float NoL, float VoH, float3 F0, float3 F90, float ThinFilmIOR, float ThinFilmTickness)
{
	// Thin film is limited (for now) to the top most layer (i.e., the upper layer need to be air)
	//					|
	//	  n1=1 / k1=0	| Air
	// ----------------- 
	//		n2 / k2=0	| Thin film
	// ----------------- 
	//		n3 / k3		| Regular material
	//					|	
	
	float Dinc = ThinFilmTickness * 10; // ThinFilmTickness is in micrometer. Range is 0 - 10um i.e, 0 - 10 000nm

	// Force eta2 -> 1.0 when Dinc -> 0.0
	const float eta2 = lerp(1.0, ThinFilmIOR, smoothstep(0.0, 0.03, Dinc));

	// Compute dot products
	const bool bIsValid = NoL > 0 && NoV > 0;

	float3 I = 0;
	if (bIsValid)
	{
		float cosTheta1 = VoH;
		float cosTheta2 = sqrt(1.0 - Pow2(1.0 / eta2) * (1 - Pow2(cosTheta1)));

		// First interface is dieletric so it is achromatic
		float R12 = F_Schlick(DielectricIorToF0(ThinFilmIOR), VoH).x;
		float phi12 = 0;
		float R21 = R12;
		float T121 = 1 - R12;
		float phi21 = PI - phi12;

		// Second interface is either conductor or dieletric, so can be chromatic. 
		// Ideally we should recompute F0 (R23) by taking into account the thin layer IOR
		float3 R23 = F0;
		float phi23 = 0;

		// Phase shift
		float OPD = Dinc * cosTheta2;
		float phi2 = phi21 + phi23;

		// Compound terms
		float3 R123 = R12 * R23;
		float3 r123 = sqrt(R123);
		float3 Rs = Pow2(T121) * R23 / (1 - R123);

		// Reflectance term for m=0 (DC term amplitude)
		float3 C0 = R12 + Rs;
		float3 S0 = ThinFilmEvalSensitivity(0.0, 0.0);
		I += C0 * S0;

		// Reflectance term for m>0 (pairs of diracs)
		// Simplified based as we don't take into account polarization
		float3 Cm = Rs - T121;
		for (int m = 1; m <= 3; ++m)
		{
			Cm *= r123;
			I += Cm * 2.0 * ThinFilmEvalSensitivity(m * OPD, m * phi2);
		}

		// Convert back to RGB reflectance
		// XYZ to CIE 1931 RGB color space (using neutral E illuminant)
		const float3x3 ThinFilm_XYZ_TO_RGB = float3x3(2.3706743, -0.5138850, 0.0052982, -0.9000405, 1.4253036, -0.0146949, -0.4706338, 0.0885814, 1.0093968);
		I = saturate(mul(I, ThinFilm_XYZ_TO_RGB));
	}
	return I;
}