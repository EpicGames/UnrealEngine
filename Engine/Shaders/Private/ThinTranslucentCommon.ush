// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShadingCommon.ush"

#pragma once

#define THIN_TRANSLUCENT_USE_DOTNV_THICKNESS 1

#if MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT || MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE && (MATERIAL_SHADINGMODEL_THIN_TRANSLUCENT && !TEMPLATE_USES_SUBSTRATE)
void AccumulateThinTranslucentModel(inout float3 DualBlendSurfaceLuminancePostCoverage,
								inout float3 DualBlendSurfaceTransmittancePreCoverage,
								inout float DualBlendSurfaceCoverage,
								inout FMaterialPixelParameters MaterialParams,
								FGBufferData GBuffer,
								float3 DiffuseColor,
								float3 SpecularColor,
								float3 EmissiveColor,
								float TopMaterialCoverage)
{
	const float3 N = MaterialParams.WorldNormal;
	const float3 V = MaterialParams.CameraVector;

	const float NoV = saturate( abs( dot(N, V) ) + 1e-5 );

	// how much to multiply the background color by
	float3 Transmittance = float3(1.0,1.0,1.0);

	// how much to add for foreground color
	float3 SurfaceColor = float3(0.0f,0.0f,0.0f);

	const float3 TransmittanceColor = GetThinTranslucentMaterialOutput0(MaterialParams);
	const float  SurfaceCoverage    = GetThinTranslucentMaterialOutput1(MaterialParams);
	
	// color is for normalized thickness
#if THIN_TRANSLUCENT_USE_DOTNV_THICKNESS
    // TODO: in theory this should account for the IOR as well, see the path tracing code for a reference
	// However in practice the difference is subtle, so keep using this simple logic for now.
	float PathLength = rcp(NoV);
#else
	float PathLength = 1.0f;
#endif	
	float3 NegativeAbsorptionCoefficient = log(TransmittanceColor);
	float3 BottomMaterialTransmittance = exp(NegativeAbsorptionCoefficient * PathLength);

	// Light goes from background -> solid surface -> camera, and we need fresnel at both interactions.
	const float3 FresnelRatio = F_Schlick(GBuffer.SpecularColor, NoV);

	// FresnelRatio light is reflected back into the background, and the rest refracts into the surface.
	Transmittance = Transmittance * (1-FresnelRatio);

	// Light gets lost from absorption through the surface
	Transmittance = Transmittance * BottomMaterialTransmittance;

	// Exiting the surface, the Fresnel ratio is the same! This is technically only true for F_Fresnel, but since we are using
	// F_Schlick as a stand-in, applying the same logic is a valid approximation
	// In mathematica, using fr[cos,eta] we can verify that:
	//   fr[c_, eta_] := 1 / 2 * A ^ 2 * (1 + B ^ 2) /.A -> (g - c) / (g + c) /.B -> (c * (g + c) - 1) / (c * (g - c) + 1) /.g->Sqrt[eta ^ 2 - 1 + c ^ 2];
	//   FullSimplify[fr[c, 1/ior] == fr[Sqrt[1 - (1 - c ^ 2) * ior ^ 2], ior], Assumptions->ior > 1 && c > 0 && c < 1]
	// evaluates to
	//   True
	Transmittance = Transmittance * (1-FresnelRatio);

	// We are treating the BaseColor and Emissive color as a layer on top of the absorbing media, but below specular layer.
	float3 DefaultLitColor = DiffuseColor + EmissiveColor;

	SurfaceColor += DefaultLitColor * TopMaterialCoverage;
	Transmittance *= (1.0f - TopMaterialCoverage);

	SurfaceColor += SpecularColor;

	// Luminance and transmittance assuming a full coverage of 1.
	DualBlendSurfaceCoverage = SurfaceCoverage;
	DualBlendSurfaceLuminancePostCoverage = SurfaceColor * DualBlendSurfaceCoverage;
	DualBlendSurfaceTransmittancePreCoverage = Transmittance;
}
#endif


