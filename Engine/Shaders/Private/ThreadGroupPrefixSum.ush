// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * ThreadGroupPrefixSum(Value, GroupThreadIndex [, GroupSumOut])
 *
 * Calculates the exclusive prefix sum (and optionally group sum) of a given value across threads of a thread group.
 *
 * EXAMPLE USES:
 *	float GroupSum;
 *	float PrefixSum = ThreadGroupPrefixSum(ValueFloat, GroupThreadIndex, GroupSum);
 *
 *	int PrefixSum = ThreadGroupPrefixSum(ValueInt, GroupThreadIndex);
 *
 * NOTES:
 * - (Exclusive) Prefix Sum means the sum of the value for all threads whose group thread index is LESS than the current
 * - Only scalar types are currently supported
 * - All threads in the group must be active when calling this method; it cannot be used in a branch (or after an early return)
 *	 that doesn't include the entire group.
 */

#ifndef NUM_THREADS_PER_GROUP
	#error NUM_THREADS_PER_GROUP must be defined, and must be equal to the thread group size of the caller of ThreadGroupPrefixSum
#endif

// re-use the same groupshared memory, in case the caller utilizes multiple overloads
groupshared uint ThreadGroupPrefixSumWorkspace[2][NUM_THREADS_PER_GROUP];

#define DECLARE_THREAD_GROUP_PREFIX_SUM(ValType, CastToUint, CastFromUint) \
	ValType ThreadGroupPrefixSum(ValType Value, uint GroupThreadIndex, inout ValType GroupSum) \
	{ \
		uint Curr = 0; \
		ThreadGroupPrefixSumWorkspace[Curr][GroupThreadIndex] = CastToUint(Value); \
		GroupMemoryBarrierWithGroupSync(); \
		for (uint i = 1U; i <= (NUM_THREADS_PER_GROUP / 2U); i *= 2U) \
		{ \
			const uint Next = 1U - Curr; \
			if (GroupThreadIndex < i) \
			{ \
				ThreadGroupPrefixSumWorkspace[Next][GroupThreadIndex] = \
					ThreadGroupPrefixSumWorkspace[Curr][GroupThreadIndex]; \
			} \
			else \
			{ \
				ThreadGroupPrefixSumWorkspace[Next][GroupThreadIndex] = CastToUint( \
					CastFromUint(ThreadGroupPrefixSumWorkspace[Curr][GroupThreadIndex]) + \
					CastFromUint(ThreadGroupPrefixSumWorkspace[Curr][GroupThreadIndex - i]) \
				); \
			} \
			Curr = Next; \
			GroupMemoryBarrierWithGroupSync(); \
		} \
		GroupSum = CastFromUint(ThreadGroupPrefixSumWorkspace[Curr][NUM_THREADS_PER_GROUP - 1]); \
		if (GroupThreadIndex == 0U) \
		{ \
			return CastFromUint(0); \
		} \
		else \
		{ \
			return CastFromUint(ThreadGroupPrefixSumWorkspace[Curr][GroupThreadIndex - 1]); \
		} \
	} \
	ValType ThreadGroupPrefixSum(ValType Value, uint GroupThreadIndex) \
	{ \
		ValType GroupSum; \
		return ThreadGroupPrefixSum(Value, GroupThreadIndex, GroupSum); \
	}

DECLARE_THREAD_GROUP_PREFIX_SUM(float, asuint, asfloat)
DECLARE_THREAD_GROUP_PREFIX_SUM(int, uint, int)
DECLARE_THREAD_GROUP_PREFIX_SUM(uint, uint, uint)


#define DECLARE_THREAD_GROUP_REDUCE_SUM(ValType, CastToUint, CastFromUint) \
	ValType ThreadGroupReduceSum(ValType Value, uint GroupThreadIndex) \
	{ \
		ValType Result = Value; \
		ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Value); \
		GroupMemoryBarrierWithGroupSync(); \
		for (uint Index = NUM_THREADS_PER_GROUP / 2U; Index > 0U; Index >>= 1U) \
		{ \
			if (GroupThreadIndex < Index) \
			{ \
				Result += CastFromUint(ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex + Index]); \
				ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Result); \
			} \
			GroupMemoryBarrierWithGroupSync(); \
		} \
		return CastFromUint(ThreadGroupPrefixSumWorkspace[0][0]); \
	}

DECLARE_THREAD_GROUP_REDUCE_SUM(float, asuint, asfloat)
DECLARE_THREAD_GROUP_REDUCE_SUM(int, uint, int)
DECLARE_THREAD_GROUP_REDUCE_SUM(uint, uint, uint)

#undef DECLARE_THREAD_GROUP_REDUCE_SUM


#define DECLARE_THREAD_GROUP_REDUCE_MAX(ValType, CastToUint, CastFromUint) \
	ValType ThreadGroupReduceMax(ValType Value, uint GroupThreadIndex) \
	{ \
		ValType Result = Value; \
		ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Value); \
		GroupMemoryBarrierWithGroupSync(); \
		for (uint Index = NUM_THREADS_PER_GROUP / 2U; Index > 0U; Index >>= 1U) \
		{ \
			if (GroupThreadIndex < Index) \
			{ \
				Result = max(Result, CastFromUint(ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex + Index])); \
				ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Result); \
			} \
			GroupMemoryBarrierWithGroupSync(); \
		} \
		return CastFromUint(ThreadGroupPrefixSumWorkspace[0][0]); \
	} 


DECLARE_THREAD_GROUP_REDUCE_MAX(float, asuint, asfloat)
DECLARE_THREAD_GROUP_REDUCE_MAX(int, uint, int)
DECLARE_THREAD_GROUP_REDUCE_MAX(uint, uint, uint)

#undef DECLARE_THREAD_GROUP_REDUCE_MAX

#define DECLARE_THREAD_GROUP_REDUCE_MIN(ValType, CastToUint, CastFromUint) \
	ValType ThreadGroupReduceMin(ValType Value, uint GroupThreadIndex) \
	{ \
		ValType Result = Value; \
		ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Value); \
		GroupMemoryBarrierWithGroupSync(); \
		for (uint Index = NUM_THREADS_PER_GROUP / 2U; Index > 0U; Index >>= 1U) \
		{ \
			if (GroupThreadIndex < Index) \
			{ \
				Result = min(Result, CastFromUint(ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex + Index])); \
				ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = CastToUint(Result); \
			} \
			GroupMemoryBarrierWithGroupSync(); \
		} \
		return CastFromUint(ThreadGroupPrefixSumWorkspace[0][0]); \
	} 

DECLARE_THREAD_GROUP_REDUCE_MIN(float, asuint, asfloat)
DECLARE_THREAD_GROUP_REDUCE_MIN(int, uint, int)
DECLARE_THREAD_GROUP_REDUCE_MIN(uint, uint, uint)

uint ThreadGroupReduceOr(uint Value, uint GroupThreadIndex)
{
	uint Result = Value;
	ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = Value;
	GroupMemoryBarrierWithGroupSync();
	for (uint Index = NUM_THREADS_PER_GROUP / 2U; Index > 0U; Index >>= 1U)
	{
		if (GroupThreadIndex < Index)
		{
			Result = Result | ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex + Index];
			ThreadGroupPrefixSumWorkspace[0][GroupThreadIndex] = Result;
		}
		GroupMemoryBarrierWithGroupSync();
	}
	return ThreadGroupPrefixSumWorkspace[0][0];
} 
