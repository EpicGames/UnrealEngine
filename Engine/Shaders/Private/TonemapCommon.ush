// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TonemapCommon.usf: PostProcessing tone mapping common
=============================================================================*/

#pragma once

#ifndef USE_ACES_2
	#define USE_ACES_2 0
#endif

#include "GammaCorrectionCommon.ush"
#include "ACES/ACESCommon.ush"
#if USE_ACES_2
	#include "ACES/ACESOutputTransform.ush"
	#include "ACES/ACESDisplayEncoding.ush"
#else
	#include "ACES/ACES_v1.3.ush"
#endif

/** Values of output device (matching C++'s ETonemappingMethod) */
#define TONEMAPPING_METHOD_Filmic 0
#define TONEMAPPING_METHOD_ACES 1

/** Values of output device (matching C++'s EDisplayOutputFormat) */
#define TONEMAPPER_OUTPUT_sRGB 0
#define TONEMAPPER_OUTPUT_Rec709 1
#define TONEMAPPER_OUTPUT_ExplicitGammaMapping 2
#define TONEMAPPER_OUTPUT_ACES1000nitST2084 3
#define TONEMAPPER_OUTPUT_ACES2000nitST2084 4
#define TONEMAPPER_OUTPUT_ACES1000nitScRGB 5
#define TONEMAPPER_OUTPUT_ACES2000nitScRGB 6
#define TONEMAPPER_OUTPUT_LinearEXR 7
#define TONEMAPPER_OUTPUT_NoToneCurve 8
#define TONEMAPPER_OUTPUT_WithToneCurve 9

/** Values of output device (matching C++'s EDisplayColorGamut) */
#define TONEMAPPER_GAMUT_sRGB_D65 0
#define TONEMAPPER_GAMUT_DCIP3_D65 1
#define TONEMAPPER_GAMUT_Rec2020_D65 2
#define TONEMAPPER_GAMUT_ACES_D60 3
#define TONEMAPPER_GAMUT_ACEScg_D60 4


// usually 1/2.2, the .y is used for inverse gamma when "gamma only" mode is not used
half3 InverseGamma;

// Scale factor for converting pixel values to nits. 
// This value is required for PQ (ST2084) conversions, because PQ linear values are in nits. 
// The purpose is to make good use of PQ lut entries. A scale factor of 100 conveniently places 
// about half of the PQ lut indexing below 1.0, with the other half for input values over 1.0.
// Also, 100nits is the expected monitor brightness for a 1.0 pixel value without a tone curve.
static const float LinearToNitsScale = 100.0;
static const float LinearToNitsScaleInverse = 1.0 / 100.0;

half3 TonemapAndGammaCorrect(half3 LinearColor)
{
	// Clamp input to be positive
	// This displays negative colors as black
	LinearColor = max(LinearColor, 0);

	half3 GammaColor = pow(LinearColor, InverseGamma.x);
	
	// in all cases it's good to clamp into the 0..1 range (e.g for LUT color grading)
	GammaColor = saturate(GammaColor);

	return GammaColor;
}

uint TonemappingMethod;

/*
============================================
// Uncharted settings
Slope = 0.63;
Toe = 0.55;
Shoulder = 0.47;
BlackClip= 0;
WhiteClip = 0.01;

// HP settings
Slope = 0.65;
Toe = 0.63;
Shoulder = 0.45;
BlackClip = 0;
WhiteClip = 0;

// Legacy settings
Slope = 0.98;
Toe = 0.3;
Shoulder = 0.22;
BlackClip = 0;
WhiteClip = 0.025;

// ACES settings
Slope = 0.91;
Toe = 0.53;
Shoulder = 0.23;
BlackClip = 0;
WhiteClip = 0.035;
===========================================
*/
float FilmSlope;
float FilmToe;
float FilmShoulder;
float FilmBlackClip;
float FilmWhiteClip;

half3 FilmToneMap( half3 LinearColor ) 
{
	const float3x3 sRGB_2_AP0 = mul( XYZ_2_AP0_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );
	const float3x3 sRGB_2_AP1 = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );

	const float3x3 AP0_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP0_2_XYZ_MAT ) );
	const float3x3 AP1_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	
	const float3x3 AP0_2_AP1 = mul( XYZ_2_AP1_MAT, AP0_2_XYZ_MAT );
	const float3x3 AP1_2_AP0 = mul( XYZ_2_AP0_MAT, AP1_2_XYZ_MAT );
	
	float3 ColorAP1 = LinearColor;
	//float3 ColorAP1 = mul( sRGB_2_AP1, float3(LinearColor) );

#if 0
	{
		float3 oces = Inverse_ODT_sRGB_D65( LinearColor );
		float3 aces = Inverse_RRT( oces );
		ColorAP1 = mul( AP0_2_AP1, aces );
	}
#endif
	
#if 0
	float3 ColorSRGB = mul( AP1_2_sRGB, ColorAP1 );
	ColorSRGB = max( 0, ColorSRGB );
	ColorAP1 = mul( sRGB_2_AP1, ColorSRGB );
#endif

	float3 ColorAP0 = mul( AP1_2_AP0, ColorAP1 );

#if 0
	{
		float3 aces = ColorAP0;
		float3 oces = RRT( aces );
		LinearColor = ODT_sRGB_D65( oces );
	}
	return mul( sRGB_2_AP1, LinearColor );
#endif

#if 1
	// "Glow" module constants
	const float RRT_GLOW_GAIN = 0.05;
	const float RRT_GLOW_MID = 0.08;

	float saturation = rgb_2_saturation( ColorAP0 );
	float ycIn = rgb_2_yc( ColorAP0 );
	float s = sigmoid_shaper( (saturation - 0.4) / 0.2);
	float addedGlow = 1 + glow_fwd( ycIn, RRT_GLOW_GAIN * s, RRT_GLOW_MID);
	ColorAP0 *= addedGlow;
#endif

#if 1
	// --- Red modifier --- //
	const float RRT_RED_SCALE = 0.82;
	const float RRT_RED_PIVOT = 0.03;
	const float RRT_RED_HUE = 0;
	const float RRT_RED_WIDTH = 135;
	float hue = rgb_2_hue( ColorAP0 );
	float centeredHue = center_hue( hue, RRT_RED_HUE );
	float hueWeight = Square( smoothstep( 0, 1, 1 - abs( 2 * centeredHue / RRT_RED_WIDTH ) ) );
		
	ColorAP0.r += hueWeight * saturation * (RRT_RED_PIVOT - ColorAP0.r) * (1. - RRT_RED_SCALE);
#endif
	
	// Use ACEScg primaries as working space
	float3 WorkingColor = mul( AP0_2_AP1_MAT, ColorAP0 );

	WorkingColor = max( 0, WorkingColor );

	// Pre desaturate
	WorkingColor = lerp( dot( WorkingColor, AP1_RGB2Y ), WorkingColor, 0.96 );
	
	const half ToeScale			= 1 + FilmBlackClip - FilmToe;
	const half ShoulderScale	= 1 + FilmWhiteClip - FilmShoulder;
	
	const float InMatch = 0.18;
	const float OutMatch = 0.18;

	float ToeMatch;
	if( FilmToe > 0.8 )
	{
		// 0.18 will be on straight segment
		ToeMatch = ( 1 - FilmToe  - OutMatch ) / FilmSlope + log10( InMatch );
	}
	else
	{
		// 0.18 will be on toe segment

		// Solve for ToeMatch such that input of InMatch gives output of OutMatch.
		const float bt = ( OutMatch + FilmBlackClip ) / ToeScale - 1;
		ToeMatch = log10( InMatch ) - 0.5 * log( (1+bt)/(1-bt) ) * (ToeScale / FilmSlope);
	}

	float StraightMatch = ( 1 - FilmToe ) / FilmSlope - ToeMatch;
	float ShoulderMatch = FilmShoulder / FilmSlope - StraightMatch;
	
	half3 LogColor = log10( WorkingColor );
	half3 StraightColor = FilmSlope * ( LogColor + StraightMatch );
	
	half3 ToeColor		= (    -FilmBlackClip ) + (2 *      ToeScale) / ( 1 + exp( (-2 * FilmSlope /      ToeScale) * ( LogColor -      ToeMatch ) ) );
	half3 ShoulderColor	= ( 1 + FilmWhiteClip ) - (2 * ShoulderScale) / ( 1 + exp( ( 2 * FilmSlope / ShoulderScale) * ( LogColor - ShoulderMatch ) ) );

	ToeColor		= select(LogColor <      ToeMatch,      ToeColor, StraightColor);
	ShoulderColor	= select(LogColor > ShoulderMatch, ShoulderColor, StraightColor);

	half3 t = saturate( ( LogColor - ToeMatch ) / ( ShoulderMatch - ToeMatch ) );
	t = ShoulderMatch < ToeMatch ? 1 - t : t;
	t = (3-2*t)*t*t;
	half3 ToneColor = lerp( ToeColor, ShoulderColor, t );

	// Post desaturate
	ToneColor = lerp( dot( float3(ToneColor), AP1_RGB2Y ), ToneColor, 0.93 );

	// Returning positive AP1 values
	return max( 0, ToneColor );
}

half3 FilmToneMapInverse( half3 ToneColor ) 
{
	const float3x3 sRGB_2_AP1 = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );
	const float3x3 AP1_2_sRGB = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	
	// Use ACEScg primaries as working space
	half3 WorkingColor = mul( sRGB_2_AP1, saturate( ToneColor ) );

	WorkingColor = max( 0, WorkingColor );
	
	// Post desaturate
	WorkingColor = lerp( dot( WorkingColor, AP1_RGB2Y ), WorkingColor, 1.0 / 0.93 );

	half3 ToeColor		= 0.374816 * pow( 0.9 / min( WorkingColor, 0.8 ) - 1, -0.588729 );
	half3 ShoulderColor	= 0.227986 * pow( 1.56 / ( 1.04 - WorkingColor ) - 1, 1.02046 );

	half3 t = saturate( ( WorkingColor - 0.35 ) / ( 0.45 - 0.35 ) );
	t = (3-2*t)*t*t;
	half3 LinearColor = lerp( ToeColor, ShoulderColor, t );

	// Pre desaturate
	LinearColor = lerp( dot( LinearColor, AP1_RGB2Y ), LinearColor, 1.0 / 0.96 );

#if WORKING_COLOR_SPACE_IS_SRGB
	LinearColor = mul( AP1_2_sRGB, LinearColor );
#else
	LinearColor = mul( XYZ_TO_RGB_WORKING_COLOR_SPACE_MAT, mul( AP1_2_XYZ_MAT, LinearColor ) );
#endif
	// Returning positive working color space values
	return max( 0, LinearColor );
}

WhiteStandard GetWhiteStandard(uint OutputGamut)
{
	if (OutputGamut == TONEMAPPER_GAMUT_DCIP3_D65)
		return WhiteStandard_D65;
	else if (OutputGamut == TONEMAPPER_GAMUT_Rec2020_D65)
		return WhiteStandard_D65;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACES_D60)
		return WhiteStandard_D60;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACEScg_D60)
		return WhiteStandard_D60;
	else
		return WhiteStandard_D65;
}

float3x3 XYZtoRGBMatrix(uint OutputGamut)
{
	if (OutputGamut == TONEMAPPER_GAMUT_DCIP3_D65)
		return XYZ_2_P3D65_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_Rec2020_D65)
		return XYZ_2_Rec2020_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACES_D60)
		return XYZ_2_AP0_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACEScg_D60)
		return XYZ_2_AP1_MAT;
	else
		return XYZ_2_sRGB_MAT;
}

float3x3 RGBtoXYZMatrix(uint OutputGamut)
{
	if (OutputGamut == TONEMAPPER_GAMUT_DCIP3_D65)
		return P3D65_2_XYZ_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_Rec2020_D65)
		return Rec2020_2_XYZ_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACES_D60)
		return AP0_2_XYZ_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACEScg_D60)
		return AP1_2_XYZ_MAT;
	else
		return sRGB_2_XYZ_MAT;
}

FColorSpace GetColorSpace(uint OutputGamut)
{
	FColorSpace OutColorSpace;
	OutColorSpace.White = GetWhiteStandard(OutputGamut);
	OutColorSpace.XYZtoRGB = XYZtoRGBMatrix(OutputGamut);
	OutColorSpace.RGBtoXYZ = RGBtoXYZMatrix(OutputGamut);
	return OutColorSpace;
}

static const int UE_ACES_EOTF = 4;          // 0: ST-2084 (PQ)
											// 1: BT.1886 (Rec.709/2020 settings) 
											// 2: sRGB (mon_curve w/ presets)
											// 3: gamma 2.6
											// 4: linear (no EOTF)
											// 5: HLG

static const int UE_ACES_SURROUND = 0;      // 0: dark ( NOTE: this is the only active setting! )
											// 1: dim ( *inactive* - selecting this will have no effect )
											// 2: normal ( *inactive* - selecting this will have no effect )

static const bool UE_ACES_STRETCH_BLACK = true; // stretch black luminance to a PQ code value of 0
static const bool UE_ACES_D60_SIM = false;
static const bool UE_ACES_LEGAL_RANGE = false;

#if USE_ACES_2
float3 ACESOutputTransform(
	float3 SceneReferredLinearColor,
	const float3x3 WCS_2_AP0,
	float PeakLuminance,
	Texture2D<float> ReachTable,
	Texture2D<float> GamutTable,
	Texture2D<float> GammaTable,
	bool bScaleWhite,
	float3x3 LIMIT_RGB_TO_XYZ,
	float3x3 LIMIT_XYZ_TO_RGB,
	FColorSpace EncodingColorSpace
)
{
	float3 Aces = mul( WCS_2_AP0, SceneReferredLinearColor );

	ODTParams PARAMS = init_ODTParams(
		PeakLuminance,
		LIMIT_RGB_TO_XYZ,
		LIMIT_XYZ_TO_RGB,
		EncodingColorSpace.RGBtoXYZ,
		EncodingColorSpace.XYZtoRGB,
		ReachTable,
		GamutTable,
		GammaTable
	);

	float3 ColorXYZ = outputTransform_fwd(Aces, PARAMS);

	// Clamp to peak luminance
	ColorXYZ = clamp_zero_to_peakLuminance(ColorXYZ, PARAMS);

	if (bScaleWhite)
	{
		ColorXYZ = apply_white_scale(ColorXYZ, PARAMS, false);
	}

	// Scale by reference luminance to return values in nits, matching our 1.3 transform
	return mul(EncodingColorSpace.XYZtoRGB, ColorXYZ) * referenceLuminance;
}

float3 ACESInvOutputTransform(
	float3 EncodingNits,
	const float3x3 AP0_2_WCS,
	float PeakLuminance,
	Texture2D<float> ReachTable,
	Texture2D<float> GamutTable,
	Texture2D<float> GammaTable,
	const bool bScaleWhite,
	float3x3 LIMIT_RGB_TO_XYZ,
	float3x3 LIMIT_XYZ_TO_RGB,
	FColorSpace EncodingColorSpace
)
{
	float3 ColorXYZ = mul(EncodingColorSpace.RGBtoXYZ, EncodingNits / referenceLuminance);

	ODTParams PARAMS = init_ODTParams(
		PeakLuminance,
		LIMIT_RGB_TO_XYZ,
		LIMIT_XYZ_TO_RGB,
		EncodingColorSpace.RGBtoXYZ,
		EncodingColorSpace.XYZtoRGB,
		ReachTable,
		GamutTable,
		GammaTable
	);

	if (bScaleWhite)
	{
		ColorXYZ = apply_white_scale(ColorXYZ, PARAMS, true);
	}

	// Clamp to peak luminance
	ColorXYZ = clamp_zero_to_peakLuminance(ColorXYZ, PARAMS);

	float3 Aces = outputTransform_inv(ColorXYZ, PARAMS);

	return mul(AP0_2_WCS, Aces);
}

#else // USE_ACES_2
struct FACESTonemapParams
{
	TsParams AcesParams;
	float SceneColorMultiplier;
	float GamutCompression;
};

FACESTonemapParams ComputeACESTonemapParams(float4 InACESMinMaxData, float4 InACESMidData, float4 InACESCoefsLow_0, float4 InACESCoefsHigh_0, float InACESCoefsLow_4, float InACESCoefsHigh_4, float SceneColorMultiplier, float GamutCompression)
{
	FACESTonemapParams OutACESTonemapParams;
	OutACESTonemapParams.AcesParams.Min.x = InACESMinMaxData.x;
	OutACESTonemapParams.AcesParams.Min.y = InACESMinMaxData.y;
	OutACESTonemapParams.AcesParams.Min.slope = 0;

	OutACESTonemapParams.AcesParams.Mid.x = InACESMidData.x;
	OutACESTonemapParams.AcesParams.Mid.y = InACESMidData.y;
	OutACESTonemapParams.AcesParams.Mid.slope = InACESMidData.z;

	OutACESTonemapParams.AcesParams.Max.x = InACESMinMaxData.z;
	OutACESTonemapParams.AcesParams.Max.y = InACESMinMaxData.w;
	OutACESTonemapParams.AcesParams.Max.slope = 0;

	OutACESTonemapParams.AcesParams.coefsLow[0] = InACESCoefsLow_0.x;
	OutACESTonemapParams.AcesParams.coefsLow[1] = InACESCoefsLow_0.y;
	OutACESTonemapParams.AcesParams.coefsLow[2] = InACESCoefsLow_0.z;
	OutACESTonemapParams.AcesParams.coefsLow[3] = InACESCoefsLow_0.w;
	OutACESTonemapParams.AcesParams.coefsLow[4] = InACESCoefsLow_4;
	OutACESTonemapParams.AcesParams.coefsLow[5] = InACESCoefsLow_4;

	OutACESTonemapParams.AcesParams.coefsHigh[0] = InACESCoefsHigh_0.x;
	OutACESTonemapParams.AcesParams.coefsHigh[1] = InACESCoefsHigh_0.y;
	OutACESTonemapParams.AcesParams.coefsHigh[2] = InACESCoefsHigh_0.z;
	OutACESTonemapParams.AcesParams.coefsHigh[3] = InACESCoefsHigh_0.w;
	OutACESTonemapParams.AcesParams.coefsHigh[4] = InACESCoefsHigh_4;
	OutACESTonemapParams.AcesParams.coefsHigh[5] = InACESCoefsHigh_4;

	OutACESTonemapParams.SceneColorMultiplier = SceneColorMultiplier;
	OutACESTonemapParams.GamutCompression = GamutCompression;

	return OutACESTonemapParams;

}

//
// ACES D65 nit Output Transform - Forward
//  Input is scene-referred linear values in the working space gamut
//  Output is output-referred linear values in the AP1 gamut
//
float3 ACESOutputTransform( float3 SceneReferredLinearColor, const float3x3 WCS_2_AP0, FACESTonemapParams InACESTonemapParams)
{
	float3 aces = mul( WCS_2_AP0, SceneReferredLinearColor * InACESTonemapParams.SceneColorMultiplier);
	float3 AcesCompressed = compressColor(aces);
	aces = lerp(aces, AcesCompressed, InACESTonemapParams.GamutCompression);

	const FColorSpace UE_ACES_AP1_DISPLAY_PRI = GetColorSpace(TONEMAPPER_GAMUT_ACEScg_D60);
	const FColorSpace UE_ACES_AP1_LIMITING_PRI = GetColorSpace(TONEMAPPER_GAMUT_ACEScg_D60);
	float3 OutputReferredLinearAP1Color = outputTransform(aces, InACESTonemapParams.AcesParams, UE_ACES_AP1_DISPLAY_PRI, UE_ACES_AP1_LIMITING_PRI, UE_ACES_EOTF,
											 			  UE_ACES_SURROUND, UE_ACES_STRETCH_BLACK, UE_ACES_D60_SIM, UE_ACES_LEGAL_RANGE);

	return OutputReferredLinearAP1Color;
}

float3 ACESOutputTransform( float3 SceneReferredLinearsRGBColor, FACESTonemapParams InACESTonemapParams)
{
	const float3x3 sRGB_2_AP0 = mul( XYZ_2_AP0_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );

	return ACESOutputTransform( SceneReferredLinearsRGBColor, sRGB_2_AP0, InACESTonemapParams);
}
#endif //USE_ACES_2

static const float3x3 GamutMappingIdentityMatrix = { 1.0, 0.0, 0.0, 0.0, 1.0, 0.0, 0.0, 0.0, 1.0 };

//
// Gamut conversion matrices
//
float3x3 OutputGamutMappingMatrix( uint OutputGamut )
{
	// Gamut mapping matrices used later
	const float3x3 AP1_2_sRGB    = mul( XYZ_2_sRGB_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	const float3x3 AP1_2_DCI_D65 = mul( XYZ_2_P3D65_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );
	const float3x3 AP1_2_Rec2020 = mul( XYZ_2_Rec2020_MAT, mul( D60_2_D65_CAT, AP1_2_XYZ_MAT ) );

	// Set gamut mapping matrix 
	// 0 = sRGB - D65
	// 1 = P3 - D65
	// 2 = Rec.2020 - D65
	// 3 = ACES AP0 - D60
	// 4 = ACES AP1 - D60

	float3x3 GamutMappingMatrix = AP1_2_sRGB;
	if( OutputGamut == TONEMAPPER_GAMUT_DCIP3_D65)
		GamutMappingMatrix = AP1_2_DCI_D65;
	else if( OutputGamut == TONEMAPPER_GAMUT_Rec2020_D65)
		GamutMappingMatrix = AP1_2_Rec2020;
	else if( OutputGamut == TONEMAPPER_GAMUT_ACES_D60)
		GamutMappingMatrix = AP1_2_AP0_MAT;
	else if( OutputGamut == TONEMAPPER_GAMUT_ACEScg_D60)
		GamutMappingMatrix = GamutMappingIdentityMatrix;

	return GamutMappingMatrix; 
}

float3x3 OutputInverseGamutMappingMatrix( uint OutputGamut )
{
	// Gamut mapping matrices used later
	const float3x3 sRGB_2_AP1    = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, sRGB_2_XYZ_MAT ) );
	const float3x3 DCI_D65_2_AP1 = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, P3D65_2_XYZ_MAT ) );
	const float3x3 Rec2020_2_AP1 = mul( XYZ_2_AP1_MAT, mul( D65_2_D60_CAT, Rec2020_2_XYZ_MAT ) );

	// Set gamut mapping matrix 
	// 0 = sRGB - D65
	// 1 = P3 (DCI) - D65
	// 2 = Rec.2020 - D65
	// 3 = ACES AP0 - D60
	float3x3 GamutMappingMatrix = sRGB_2_AP1;
	if( OutputGamut == TONEMAPPER_GAMUT_DCIP3_D65)
		GamutMappingMatrix = DCI_D65_2_AP1;
	else if( OutputGamut == TONEMAPPER_GAMUT_Rec2020_D65)
		GamutMappingMatrix = Rec2020_2_AP1;
	else if( OutputGamut == TONEMAPPER_GAMUT_ACES_D60)
		GamutMappingMatrix = AP0_2_AP1_MAT;
	else if (OutputGamut == TONEMAPPER_GAMUT_ACEScg_D60)
		GamutMappingMatrix = GamutMappingIdentityMatrix;

	return GamutMappingMatrix;
}

// Accurate for 1000K < Temp < 15000K
// [Krystek 1985, "An algorithm to calculate correlated colour temperature"]
float2 PlanckianLocusChromaticity( float Temp )
{
	float u = ( 0.860117757f + 1.54118254e-4f * Temp + 1.28641212e-7f * Temp*Temp ) / ( 1.0f + 8.42420235e-4f * Temp + 7.08145163e-7f * Temp*Temp );
	float v = ( 0.317398726f + 4.22806245e-5f * Temp + 4.20481691e-8f * Temp*Temp ) / ( 1.0f - 2.89741816e-5f * Temp + 1.61456053e-7f * Temp*Temp );

	float x = 3*u / ( 2*u - 8*v + 4 );
	float y = 2*v / ( 2*u - 8*v + 4 );

	return float2(x,y);
}

// Accurate for 4000K < Temp < 25000K
// in: correlated color temperature
// out: CIE 1931 chromaticity
float2 D_IlluminantChromaticity( float Temp )
{
	// Correct for revision of Plank's law
	// This makes 6500 == D65
	Temp *= 1.4388 / 1.438;
	float OneOverTemp = 1.0/Temp;
	float x =	Temp <= 7000 ?
				0.244063 + ( 0.09911e3 + ( 2.9678e6 - 4.6070e9 * OneOverTemp ) * OneOverTemp) * OneOverTemp:
				0.237040 + ( 0.24748e3 + ( 1.9018e6 - 2.0064e9 * OneOverTemp ) * OneOverTemp ) * OneOverTemp;
	
	float y = -3 * x*x + 2.87 * x - 0.275;

	return float2(x,y);
}

// Find closest color temperature to chromaticity
// [McCamy 1992, "Correlated color temperature as an explicit function of chromaticity coordinates"]
float CorrelatedColorTemperature( float x, float y )
{
	float n = (x - 0.3320) / (y - 0.1858);
	return -449 * n*n*n + 3525 * n*n - 6823.3 * n + 5520.33;
}

float2 PlanckianIsothermal( float Temp, float Tint )
{
	float u = ( 0.860117757f + 1.54118254e-4f * Temp + 1.28641212e-7f * Temp*Temp ) / ( 1.0f + 8.42420235e-4f * Temp + 7.08145163e-7f * Temp*Temp );
	float v = ( 0.317398726f + 4.22806245e-5f * Temp + 4.20481691e-8f * Temp*Temp ) / ( 1.0f - 2.89741816e-5f * Temp + 1.61456053e-7f * Temp*Temp );

	float ud = ( -1.13758118e9f - 1.91615621e6f * Temp - 1.53177f * Temp*Temp ) / Square( 1.41213984e6f + 1189.62f * Temp + Temp*Temp );
	float vd = (  1.97471536e9f - 705674.0f * Temp - 308.607f * Temp*Temp ) / Square( 6.19363586e6f - 179.456f * Temp + Temp*Temp );

	float2 uvd = normalize(float2(ud, vd));

	// Correlated color temperature is meaningful within +/- 0.05
	u +=  uvd.y * Tint * 0.05;
	v += -uvd.x * Tint * 0.05;

	float x = 3*u / ( 2*u - 8*v + 4 );
	float y = 2*v / ( 2*u - 8*v + 4 );

	return float2(x,y);
}

float3 WhiteBalance(float3 LinearColor, float WhiteTemp, float WhiteTint, bool bIsTemperatureWhiteBalance, const float3x3 WCS_2_XYZ, const float3x3 XYZ_2_WCS)
{
	float2 SrcWhiteDaylight = D_IlluminantChromaticity(WhiteTemp);
	float2 SrcWhitePlankian = PlanckianLocusChromaticity(WhiteTemp);

	float2 SrcWhite = WhiteTemp < 4000 ? SrcWhitePlankian : SrcWhiteDaylight;
	float2 D65White = float2(0.31270, 0.32900);

	{
		// Offset along isotherm
		float2 Isothermal = PlanckianIsothermal(WhiteTemp, WhiteTint) - SrcWhitePlankian;
		SrcWhite += Isothermal;
	}

	if (!bIsTemperatureWhiteBalance)
	{
		float2 Temp = SrcWhite;
		SrcWhite = D65White;
		D65White = Temp;
	}

	float3x3 WhiteBalanceMat = ChromaticAdaptation(SrcWhite, D65White);
	WhiteBalanceMat = mul( XYZ_2_WCS, mul( WhiteBalanceMat, WCS_2_XYZ ) );

	return mul(WhiteBalanceMat, LinearColor);
}

float3 HableToneMapCurve(float3 X)
{
	const float A = 0.15f;
	const float B = 0.50f;
	const float C = 0.10f;
	const float D = 0.20f;
	const float E = 0.02f;
	const float F = 0.30f;

	return ((X*(A*X+C*B)+D*E)/(X*(A*X+B)+D*F))-E/F;
}

float3 HableToneMap(float3 LinearColor)
{
	LinearColor = max(0, LinearColor);

	const float3 W = 11.2f;
	const float3 WhiteScale = 1.0f / HableToneMapCurve(W);
	const float ExposureBias = 2.0f;

	return HableToneMapCurve(LinearColor * ExposureBias) * WhiteScale;
}

float3 SimpleReinhardToneMap(float3 LinearColor)
{
	return LinearColor / (1.0f + LinearColor);
}
