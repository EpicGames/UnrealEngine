// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ColorGradingColorHistogram.usf: Render ColorHistogram color grading view.
	Charts the intensity of the color in every pixel in the input as a histogram.
=============================================================================*/


#include "/Engine/Private/Common.ush"

#define THREADGROUP_THREADCOUNT (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)

int IntensityToIntensityBin(float NormalizedIntensity)
{
	int Bin = round(NormalizedIntensity * (PER_CHANNEL_RESOLUTION - 1));
	return Bin;
}

#ifdef BuildColorHistogramCS

Texture2D InputColorTexture;
int4 InputColorViewRect;
float2 MinMaxValue;
RWTexture2D<uint> OutputHistogramTexture;
RWStructuredBuffer<uint> OutputMetaDataBuffer;

#define NUM_CHANNELS (3)

groupshared uint LDS[PER_CHANNEL_RESOLUTION];
groupshared uint LDS_Max[NUM_CHANNELS];

void ClearLDS(uint GroupThreadIndex)
{
	for (uint Index = GroupThreadIndex; Index < PER_CHANNEL_RESOLUTION; Index += THREADGROUP_THREADCOUNT)
	{
		LDS[Index] = 0;
	}
	for (uint Index = GroupThreadIndex; Index < NUM_CHANNELS; Index += THREADGROUP_THREADCOUNT)
	{
		LDS_Max[Index] = 0;
	}
}

void FlushAndClearLDS(uint Channel, uint GroupThreadIndex)
{
	for (uint Bin = GroupThreadIndex; Bin < PER_CHANNEL_RESOLUTION; Bin += THREADGROUP_THREADCOUNT)
	{
		uint Value = LDS[Bin];

		if (Value > 0)
		{
			LDS[Bin] = 0;

			uint PrevValue;
			InterlockedAdd(OutputHistogramTexture[int2(Bin, Channel)], Value, PrevValue);
			uint CurValue = PrevValue + Value;
			InterlockedMax(LDS_Max[Channel], CurValue);
		}
	}
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void BuildColorHistogramCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	float MinValue = MinMaxValue[0];
	float MaxValue = MinMaxValue[1];

	int2 InputColorViewSize = InputColorViewRect.zw - InputColorViewRect.xy;
	int2 PixelPosition = DispatchThreadId;
	int2 BufferPixelPosition = InputColorViewRect.xy + PixelPosition;

	bool bHasValidPixel = all(PixelPosition < InputColorViewSize);

	float3 InputSample = 0;
	if (bHasValidPixel)
	{
		InputSample = InputColorTexture.Load(int3(BufferPixelPosition, 0)).rgb;
	}

	float3 NormalizedSample = (InputSample - MinValue) / (MaxValue - MinValue);

	ClearLDS(GroupThreadIndex);
	GroupMemoryBarrierWithGroupSync();

	for (int Channel = 0; Channel < NUM_CHANNELS; ++Channel)
	{
		int Bin = IntensityToIntensityBin(NormalizedSample[Channel]);

		InterlockedAdd(LDS[Bin], 1);

		GroupMemoryBarrierWithGroupSync();

		FlushAndClearLDS(Channel, GroupThreadIndex);

		GroupMemoryBarrierWithGroupSync();
	}

	if (GroupThreadIndex == 0)
	{
		for (int Channel = 0; Channel < NUM_CHANNELS; ++Channel)
		{
			InterlockedMax(OutputMetaDataBuffer[Channel], LDS_Max[Channel]);
		}
	}
}

#endif

#ifdef DrawColorHistogramCS

Texture2D<uint> HistogramTexture;
StructuredBuffer<uint> MetaDataBuffer;
RWTexture2D<float3> OutputTexture;
int4 OutputViewRect;
int4 InputColorViewRect;
uint Flags;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void DrawColorHistogramCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	int2 OutputViewSize = OutputViewRect.zw - OutputViewRect.xy;
	int2 PixelPosition = int2(DispatchThreadId.x, DispatchThreadId.y);
	int2 BufferPixelPosition = OutputViewRect.xy + PixelPosition;
	float2 UV = (float2(PixelPosition) + 0.5) / OutputViewSize;
	UV.y = 1.0 - UV.y; // flip y

	int Bin = IntensityToIntensityBin(UV.x);

	float3 Output = 0;

	bool bColorize 					= (Flags & FLAG_COLORIZE) != 0;
	bool bDrawSeparateRowPerChannel = (Flags & FLAG_DRAW_SEPARATE_ROW_PER_CHANNEL) != 0;

	if (bDrawSeparateRowPerChannel)
	{
		int Channel = int(UV.y * 3);
		UV.y = frac(UV.y * 3);

		uint NumSamples = HistogramTexture[int2(Bin, Channel)];
		uint LargestBinCount = MetaDataBuffer[Channel];

		float BarHeight = float(NumSamples) / LargestBinCount; 
		float Intensity = UV.y < BarHeight;
		//Intensity *= lerp(1.0, 0.3, saturate((BarHeight - UV.y) * 20.0));
		Output[Channel] = Intensity;

		if (!bColorize)
		{
			Output = dot(Output, 1);
		}
	}
	else
	{
		uint3 NumSamples = 0;
		NumSamples[0] = HistogramTexture[int2(Bin, 0)];
		NumSamples[1] = HistogramTexture[int2(Bin, 1)];
		NumSamples[2] = HistogramTexture[int2(Bin, 2)];

		uint3 LargestBinCount = 0;
		LargestBinCount[0] = MetaDataBuffer[0];
		LargestBinCount[1] = MetaDataBuffer[1];
		LargestBinCount[2] = MetaDataBuffer[2];

		Output = float3(NumSamples) / LargestBinCount;

		Output = UV.yyy < Output;

		if (!bColorize)
		{
			Output = dot(Output, 0.33333f);
		}
	}

	OutputTexture[BufferPixelPosition] = Output;
}

#endif
