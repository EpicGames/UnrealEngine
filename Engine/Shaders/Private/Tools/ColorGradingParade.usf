// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ColorGradingParade.usf: Render RGB Parade and Waveform color grading views.
	These views give a histogram of color channel intensity, per image column.
=============================================================================*/

#define MATERIAL_DOMAIN_UI 1 // required for LuminanceFactors()
#include "/Engine/Private/Common.ush"

#ifndef DIM_WAVE_OPS
#define DIM_WAVE_OPS 0
#endif

float2 MinMaxValue;
uint Flags;

#define QUANT_FACTOR 1024.0 // Quantization factor, used to convert float to fixed precision int
#define QUANT(x) (int((x) * QUANT_FACTOR))
#define DEQUANT(x) (float(x) / QUANT_FACTOR)

#define COLUMN_FLAG_HAS_NAN (1U << 0)
#define COLUMN_FLAG_HAS_INF (1U << 1)

int ImageColumnToHistogramColumn(int Column, int ChannelIndex, int2 InputViewRectSize)
{
	return Column*3 + ChannelIndex;
	//return ChannelIndex * InputViewRectSize.x + Column;
}

#ifdef BuildColorParadeHistogramCS

#define THREADGROUP_THREADCOUNT (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)
#define NUM_BINS_PER_THREAD (NUM_BINS / THREADGROUP_SIZE_Y)

Texture2D InputColorTexture;
RWTexture2D<uint> OutputTexture;
RWStructuredBuffer<uint> OutputColumnFlagsBuffer;
float4 InputColorViewRect;
int2 OutputHistogramSize; // x is image columns, y is histogram bins

groupshared uint Bins[NUM_BINS];

void ClearLDS(uint GroupThreadIndex)
{
	for (uint Index = GroupThreadIndex; Index < NUM_BINS; Index += THREADGROUP_THREADCOUNT)
	{
		Bins[Index] = 0;
	}
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void BuildColorParadeHistogramCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	const float MinValue = MinMaxValue.x;
	const float MaxValue = MinMaxValue.y;

	float2 InputViewRectSize = InputColorViewRect.zw - InputColorViewRect.xy;

	// Read input value. If the output graph is smaller than the input image, aggregate multiple pixels
	float3 IntensityAcc = (float3)0;
	bool bHasInf = false;
	bool bHasNan = false;
	{
		int InputColumnsPerOutputColumn = ceil(InputViewRectSize.x / OutputHistogramSize.x);
		int StartColumn = InputColorViewRect.x + DispatchThreadId.x * InputColumnsPerOutputColumn;
		int NumColumnsToRead = min(InputViewRectSize.x - StartColumn, InputColumnsPerOutputColumn);
		int EndColumn = min(StartColumn + NumColumnsToRead, InputColorViewRect.z);
		int Row = min(InputColorViewRect.y + DispatchThreadId.y, InputColorViewRect.w-1);

		for (int Column = StartColumn; Column < EndColumn; ++Column)
		{
			float3 InputValue = InputColorTexture.Load(int3(Column, Row, 0)).rgb;

			bHasInf = bHasInf || any(!IsFinite(InputValue));
			bHasNan = bHasNan || any(isnan(InputValue));
			IntensityAcc += InputValue;
		}
		IntensityAcc /= NumColumnsToRead;
	}

#if DIM_WAVE_OPS
	bHasInf = WaveActiveAnyTrue(bHasInf);
	bHasNan = WaveActiveAnyTrue(bHasNan);
#endif

	// Build local intensity histogram in LDS
	float3 HistogramValues[NUM_BINS_PER_THREAD];
	for (int ComponentIndex = 0; ComponentIndex < 3; ++ComponentIndex)
	{
		ClearLDS(GroupThreadIndex);
		GroupMemoryBarrierWithGroupSync();

		// Write histogram
		if (DispatchThreadId.y < InputViewRectSize.y)
		{
			float Component = IntensityAcc[ComponentIndex];
			float UnitIntensity = saturate((Component - MinValue) / (MaxValue - MinValue));
			float BinIndexFloat = UnitIntensity * (NUM_BINS - 1);
			int BinIndex = round(BinIndexFloat);

			int PrevValue;

			// faster but with aliasing
			//InterlockedAdd(Bins[BinIndex], QUANT(1), PrevValue);

			// slower but higher quality
			InterlockedAdd(Bins[floor(BinIndexFloat)], QUANT(1.0 - frac(BinIndexFloat)), PrevValue);
			InterlockedAdd(Bins[ceil(BinIndexFloat)], QUANT(frac(BinIndexFloat)), PrevValue);
		}

		GroupMemoryBarrierWithGroupSync();

		// Read histogram into VGPR
		for (int Bin = 0; Bin < NUM_BINS_PER_THREAD; ++Bin)
		{
			int BinIndex = (GroupThreadIndex * NUM_BINS_PER_THREAD) + Bin;
			HistogramValues[Bin][ComponentIndex] = DEQUANT(Bins[BinIndex]);
		}

		GroupMemoryBarrierWithGroupSync();
	}

	// Build global intensity histogram, accumulating histograms across threadgroups
	for (int Bin = 0; Bin < NUM_BINS_PER_THREAD; ++Bin)
	{
		int BinIndex = (GroupThreadIndex * NUM_BINS_PER_THREAD) + Bin;
		float3 BinValue = HistogramValues[Bin];

		if (any(BinValue != 0))
		{
			for (int ComponentIndex = 0; ComponentIndex < 3; ++ComponentIndex)
			{
				int HistogramColumn = ImageColumnToHistogramColumn(DispatchThreadId.x, ComponentIndex, InputViewRectSize);
				InterlockedAdd(OutputTexture[int2(HistogramColumn, BinIndex)], QUANT(BinValue[ComponentIndex]));
			}
		}
	}

	uint ColumnFlags = 0;
	ColumnFlags |= bHasNan ? COLUMN_FLAG_HAS_NAN : 0;
	ColumnFlags |= bHasInf ? COLUMN_FLAG_HAS_INF : 0;
	if (ColumnFlags != 0
#if DIM_WAVE_OPS
		&& WaveIsFirstLane()
#endif
		)
	{
		int HistogramColumn = ImageColumnToHistogramColumn(DispatchThreadId.x, 0, InputViewRectSize);
		InterlockedOr(OutputColumnFlagsBuffer[HistogramColumn], ColumnFlags);
	}
}

#endif

#ifdef DrawColorParadeCS

#define THREADGROUP_THREADCOUNT (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)

Texture2D<uint> HistogramTexture;
SamplerState HistogramTextureSampler;
StructuredBuffer<uint> ColumnFlagsBuffer;
RWTexture2D<float3> OutputTexture;
int2 InputViewSize;
int2 HistogramSize;
int4 OutputViewRect;
float Brightness;
float Gamma;
float DesaturationPower;

float SampleHistogram(const float2 UV, int Channel)
{
	float2 HistogramBufferSize = float2(HistogramSize.x * 3, HistogramSize.y);

	int InputColumnsPerOutputColumn = ceil(float(InputViewSize.x) / HistogramSize.x);
	float HistogramActualWidth = InputViewSize.x / InputColumnsPerOutputColumn;

	float HistogramColumn = ImageColumnToHistogramColumn(floor(UV.x * HistogramActualWidth), Channel, InputViewSize);
	float2 BufferUV = float2((HistogramColumn + 0.5) / HistogramBufferSize.x, UV.y);

	return DEQUANT(HistogramTexture.Load(int3(BufferUV.x * HistogramBufferSize.x, BufferUV.y * HistogramBufferSize.y, 0)).r);
}

uint GetColumnFlags(const float2 UV, int Channel)
{
	int InputColumnsPerOutputColumn = ceil(float(InputViewSize.x) / HistogramSize.x);
	float HistogramActualWidth = InputViewSize.x / InputColumnsPerOutputColumn;

	int HistogramColumn = ImageColumnToHistogramColumn(floor(UV.x * HistogramActualWidth), Channel, InputViewSize);
	return ColumnFlagsBuffer[HistogramColumn];
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void DrawColorParadeCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	if (any(OutputViewRect.xy + DispatchThreadId > OutputViewRect.zw))
	{
		return;
	}

	bool bDrawSeparateColumnPerChannel = (Flags & FLAG_DRAW_SEPARATE_COLUMN_PER_CHANNEL) != 0;
	bool bColorize 					   = (Flags & FLAG_COLORIZE) != 0;
	bool bHighlightNaNsAndInfs		   = (Flags & FLAG_HIGHLIGHT_NANS_AND_INFS) != 0;


	float2 OutputUV = (DispatchThreadId.xy + 0.5) / (OutputViewRect.zw - OutputViewRect.xy);
	float2 InputUV = OutputUV;
	InputUV.y = 1.0 - InputUV.y; // flip Y

	if (bDrawSeparateColumnPerChannel)
	{
		InputUV.x = frac(InputUV.x * 3.0);
	}
	
	float3 HistogramValue = (float3)0;
	HistogramValue.r = SampleHistogram(InputUV, 0);
	HistogramValue.g = SampleHistogram(InputUV, 1);
	HistogramValue.b = SampleHistogram(InputUV, 2);

	HistogramValue /= InputViewSize.y;

	uint ColumnFlags = GetColumnFlags(InputUV, 0);
	bool bHasNan = (ColumnFlags & COLUMN_FLAG_HAS_NAN) > 0;
	bool bHasInf = (ColumnFlags & COLUMN_FLAG_HAS_INF) > 0;
	
	float3 OutputValue = HistogramValue * Brightness;

	if (bDrawSeparateColumnPerChannel)
	{
		// Filter out the relevant RGB primary to draw for this column
		// but bleed the value a bit across channels so clipping/blown-out values show up as white
		int ComponentIndex = int(OutputUV.x * 3.0);
		float Component = OutputValue[ComponentIndex] * 0.99;
		OutputValue = pow(Component, DesaturationPower);
		OutputValue[ComponentIndex] = Component;

		if (!bColorize)
		{
			// no Luminance() here because we want to compare the intensity across columns, 
			// rather than accurately reinterpret color to luminance
			OutputValue = dot(OutputValue, 0.3333);
		}
	}
	else
	{
		if (!bColorize)
		{
			OutputValue = Luminance(OutputValue);
		}
	}

	// Tonemap
	OutputValue = pow(OutputValue, Gamma);

	if (bHighlightNaNsAndInfs)
	{
		const float3 InfColor = float3(1,1,0) * 100.0;
		const float3 NanColor = float3(1,0,1) * 100.0;
		OutputValue = bHasInf ? InfColor : OutputValue;
		OutputValue = bHasNan ? NanColor : OutputValue;
	}

	OutputTexture[OutputViewRect.xy + DispatchThreadId] = OutputValue;
}

#endif
