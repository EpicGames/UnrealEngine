// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ColorGradingVectorScope.usf: Render Vector Scope color grading view.
	This view is a scatterplot of the hue and saturation of each pixel in the image.
	To be more accurate, it is a plot of Cb and Cr in the BT.709 YCbCr representation.
=============================================================================*/

#define MATERIAL_DOMAIN_UI 1 // required for LuminanceFactors()
#include "/Engine/Private/Common.ush"

#define THREADGROUP_THREADCOUNT (THREADGROUP_SIZE_X * THREADGROUP_SIZE_Y)

float3 RGBToYCbCr(float3 In)
{
	// BT.709 
	float Y  =  0.212600 * In.r +  0.715200 * In.g +  0.072200 * In.b;
	float Cb = -0.114572 * In.r + -0.385428 * In.g +  0.500000 * In.b;
	float Cr =  0.500000 * In.r + -0.454153 * In.g + -0.045847 * In.b;
	return float3(Y, Cb, Cr);
}

float3 YCbCrToRGB(float3 In)
{
	// BT.709
	float R =  1.000000 * In.r +  0.000000 * In.g +  1.574800 * In.b;
	float G =  1.000000 * In.r + -0.187324 * In.g + -0.468124 * In.b;
	float B =  1.000000 * In.r +  1.855600 * In.g +  0.000000 * In.b;
	return float3(R, G, B);
}

float3 RGBToNormalizedYCbCr(float3 In)
{
	float3 YCbCrSample = RGBToYCbCr(In);
	float Y = YCbCrSample[0];
	float2 CbCr = float2(YCbCrSample[1], YCbCrSample[2]);

	// Convert hue and saturation to range [0;1]
	const float2 CbCrMinMax = float2(-0.5f, 0.5f);
	float2 NormalizedCbCr = (CbCr - CbCrMinMax[0]) / (CbCrMinMax[1] - CbCrMinMax[0]);

	return float3(Y, NormalizedCbCr[0], NormalizedCbCr[1]);
}

float3 NormalizedYCbCrToRGB(float3 In)
{
	float2 CbCr = In.yz - 0.5f;
	float3 YCbCr = float3(In.x, CbCr[0], CbCr[1]);

	return YCbCrToRGB(YCbCr);
}

int HueToHueBin(float Hue)
{
	int HueBin = round(Hue * (HUE_RESOLUTION - 1));
	return HueBin;
}

int SaturationToSaturationBin(float Saturation)
{
	int SaturationBin = round(Saturation * (SATURATION_RESOLUTION - 1));
	return SaturationBin;
}

#ifdef BuildVectorScopeHistogramCS

Texture2D InputColorTexture;
int4 InputColorViewRect;
float2 LuminanceFilterMinMax;
RWTexture2D<uint> OutputHistogramTexture;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void BuildVectorScopeHistogramCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	int2 InputColorViewSize = InputColorViewRect.zw - InputColorViewRect.xy;

	uint TotalPixelCount = uint(InputColorViewSize.x) * uint(InputColorViewSize.y);
	uint PixelIndex = DispatchThreadId.x;
	uint GroupStartPixelIndex = GroupId.x * THREADGROUP_THREADCOUNT;
	uint GroupEndPixelIndex = min(GroupStartPixelIndex + THREADGROUP_THREADCOUNT, TotalPixelCount);
	int2 BufferPixelPosition = InputColorViewRect.xy + int2(PixelIndex % InputColorViewSize.x, PixelIndex / InputColorViewSize.x);

	bool bHasValidPixel = PixelIndex < TotalPixelCount;

	float3 NormalizedYCbCr = 0;
	if (bHasValidPixel)
	{
		float3 RGBSample = InputColorTexture.Load(int3(BufferPixelPosition, 0)).rgb;
		NormalizedYCbCr = RGBToNormalizedYCbCr(RGBSample);
	}

	float Luma = NormalizedYCbCr[0];
	if (Luma < LuminanceFilterMinMax[0] || Luma > LuminanceFilterMinMax[1])
	{
		return;
	}

	float Hue = NormalizedYCbCr[1];
	int HueBin = HueToHueBin(Hue);
	float Saturation = NormalizedYCbCr[2];
	int SaturationBin = SaturationToSaturationBin(Saturation);
	InterlockedAdd(OutputHistogramTexture[int2(SaturationBin, HueBin)], 1);
}

#endif

#ifdef DrawVectorScopeCS

Texture2D<uint> HistogramTexture;
RWTexture2D<float3> OutputTexture;
int4 OutputViewRect;
int4 InputColorViewRect;
uint Flags;
float Rotation;
float Brightness;
float Gamma;
float DesaturationPower;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void DrawVectorScopeCS(
	uint2 DispatchThreadId : SV_DispatchThreadID,
	uint2 GroupId : SV_GroupID,
	uint2 GroupThreadId : SV_GroupThreadID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	int2 OutputViewSize = OutputViewRect.zw - OutputViewRect.xy;
	int2 PixelPosition = int2(DispatchThreadId.x, DispatchThreadId.y);
	int2 BufferPixelPosition = OutputViewRect.xy + PixelPosition;
	float2 UV = (float2(PixelPosition) + 0.5) / OutputViewSize;
	float2 Clip = UV * 2 - 1;

	if (any(PixelPosition >= OutputViewSize))
	{
		return;
	}

	// Rotate
	{
		float Rot = PI*0.5;
		float SinRot;
		float CosRot;
		sincos(Rot, SinRot, CosRot);
		Clip = float2(
			dot(float2(CosRot, -SinRot), Clip),
			dot(float2(SinRot,  CosRot), Clip)
		);
		UV = Clip * 0.5 + 0.5;
	}

	float HueNorm = UV.y;
	int HueBin = HueToHueBin(HueNorm);

	float SaturationNorm = UV.x;
	int SaturationBin = SaturationToSaturationBin(SaturationNorm);

	float3 ColorNormYCbCr = float3(
		// Arbitrary fade to white at center
		0.2 * Pow2(1.0 - length(Clip)), 
		HueNorm, SaturationNorm);
	float3 Color = clamp(NormalizedYCbCrToRGB(ColorNormYCbCr), 0, 1);
	float Intensity = dot(Color, 0.333);
	float Desat = pow(Intensity, DesaturationPower);
	Color = max(Color, (float3)Desat);

	bool bColorize = (Flags & FLAG_COLORIZE) != 0;

	uint NumSamples = HistogramTexture.Load(int3(SaturationBin, HueBin, 0)).r;

	uint2 InputColorViewSize = InputColorViewRect.zw - InputColorViewRect.xy;
	uint NumPixels = InputColorViewSize.x * InputColorViewSize.y;

	float3 Output = bColorize ? Color : Luminance(Color);

	Output *= float(NumSamples) / NumPixels;
	Output *= InputColorViewSize.x; // Somewhat arbitrary, but rescaling with input width makes brightness scale the same as in ColorParade
	Output *= Brightness;

	// Tonemap
	Output = pow(Output, Gamma);

	OutputTexture[BufferPixelPosition] = Output;
}

#endif
