// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"


//------------------------------------------------------- PARAMETERS

uint DebugModeId;
float OpaqueWorldDistance;
float OpaqueLerpWorldRange;

Texture2D<float4> SceneColorTexture;
Texture2D<float> SceneDepthTexture;
RWTexture2D<float4> SceneColorOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(8, 8, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	int2 PixelPos = View.ViewRectMinAndSize.xy + DispatchThreadId;

	float DeviceZ = SceneDepthTexture[PixelPos];
	float4 SceneColor = SceneColorTexture[PixelPos];

	float WorldDepth = ConvertFromDeviceZ(DeviceZ);

	float4 Output = SceneColor;

	BRANCH
	if (DebugModeId == 0)
	{
		float IsTranslucent = saturate((WorldDepth - OpaqueWorldDistance) * rcp(OpaqueLerpWorldRange));

		SceneColor.rgb *= 1.0 - IsTranslucent;
		SceneColor.a = IsTranslucent;
	}

	SceneColorOutput[PixelPos] = SceneColor; 
}
