// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

Texture2D InputTexture;
uint2 InputOffset;

float4 DrawTexturePS(float4 SvPosition : SV_POSITION) : SV_Target0
{
	return InputTexture.Load(uint3((uint2)SvPosition.xy + InputOffset, 0));
}