// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"


//------------------------------------------------------- CONFIG

#define TILE_SIZE 8

#if DIM_TEXTURE_TYPE == 0 // Texture2DFloatNoMSAA
	#define payload float4	

#elif DIM_TEXTURE_TYPE == 1 // Texture2DIntNoMSAA
	#define payload uint4

#elif DIM_TEXTURE_TYPE == 2 // Texture2DDepthStencilNoMSAA
	#define payload uint4
	#define SWIZZLE_STENCIL 1

#else
	#error unknown DIM_TEXTURE_TYPE

#endif


//------------------------------------------------------- PARAMETERS

Texture2D<payload> Texture;
RWTexture2D<payload> StagingOutput;


//------------------------------------------------------- ENTRY POINT

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	payload PixelValue = Texture[DispatchThreadId];
	
	#if defined(SWIZZLE_STENCIL)
		PixelValue = payload(PixelValue STENCIL_COMPONENT_SWIZZLE, 0, 0, 0);
	#endif

	StagingOutput[DispatchThreadId] = PixelValue; 
}
