// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FullscreenVertexShader.usf: Most basic multi platform full screen vertex shader
=============================================================================*/

#include "../Common.ush"

#ifndef SCREEN_VS_FOR_INSTANCED_VIEWS
	#define SCREEN_VS_FOR_INSTANCED_VIEWS 0
#endif
#ifndef SCREEN_VS_FOR_MOBILE_MULTI_VIEW
	#define SCREEN_VS_FOR_MOBILE_MULTI_VIEW 0
#endif

#if SCREEN_VS_FOR_MOBILE_MULTI_VIEW
	#include "../InstancedStereo.ush"
#endif

void MainVS(
	float2 ViewportUV : ATTRIBUTE0,
	float2 UV         : ATTRIBUTE1, // TODO: kill
	out float4 OutPosition : SV_POSITION
#if SCREEN_VS_FOR_INSTANCED_VIEWS
	,	uint InstanceId	  : SV_InstanceID
	,	out uint OutViewportId	: SV_ViewportArrayIndex
	,	out nointerpolation uint OutViewId : VIEW_ID
#elif SCREEN_VS_FOR_MOBILE_MULTI_VIEW
	,	out noperspective float2 OutUV : TEXCOORD0
#endif
	)
{
	OutPosition = float4(ViewportUVToScreenPos(ViewportUV), 0, 1);

#if SCREEN_VS_FOR_INSTANCED_VIEWS
	OutViewId = InstanceId;
	OutViewportId = OutViewId;
#endif

#if SCREEN_VS_FOR_MOBILE_MULTI_VIEW
	OutUV = UV;
#endif
}

float4 PreTransform;
void MainForPreTransform(in uint VertexId : SV_VertexID, out FScreenVertexOutput Output)
{
	Output.UV = float2(((VertexId + 1) / 3) & 1, VertexId & 1);
	Output.Position.xy = float2(Output.UV.x, 1.f - Output.UV.y) * 2.f - 1.f;
	Output.Position.zw = float2(0, 1);
	Output.Position.xy = float2(Output.Position.x * PreTransform.x + Output.Position.y * PreTransform.z, Output.Position.x * PreTransform.y + Output.Position.y * PreTransform.w);
}