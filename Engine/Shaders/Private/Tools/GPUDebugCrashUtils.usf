// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUDebugCrashUtils.usf: Shaders for crashing the GPU for debugging purposes.
=============================================================================*/

#include "../Common.ush"

RWTexture2D<uint> PageFaultUAV;

[numthreads(THREADGROUPSIZE, THREADGROUPSIZE, 1)]
void MainCS(uint3 ThreadID : SV_DispatchThreadID)
{
#if PLATFORM_BREAK_REQUESTED
	PLATFORM_BREAK();
#endif
	uint what = PageFaultUAV[ThreadID.xy] + 1; // read

	uint OldValue = 0;
	uint2 index = uint2(0, 0);
	uint2 index1 = uint2(0, 1);
	InterlockedAdd(PageFaultUAV[index], 1, OldValue);
	if (OldValue == 0)
	{
		PageFaultUAV[index1] = what;
	}

#if ASSERT_REQUESTED && PLATFORM_SUPPORTS_DIAGNOSTIC_BUFFER
	UEReportAssertWithPayload(0xBE4A3F32, uint4(1, 2, 3, 4));
#endif

#if  HANG_REQUESTED
	// Try to trigger a crash (timeout) using a deliberate infinite loop
	for (;;)
	{
		InterlockedAdd(PageFaultUAV[index], 0, OldValue);
		if (OldValue == 0)
		{
			break;
		}
	}
#endif
}
