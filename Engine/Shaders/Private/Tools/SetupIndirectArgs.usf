// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

StructuredBuffer<uint> InputCountBuffer;
RWBuffer<uint> IndirectDispatchArgsOut;
uint Multiplier;
uint Divisor;
uint InputCountOffset;

[numthreads(1, 1, 1)]
void InitIndirectArgs1DCS(uint3 DTID : SV_DispatchThreadID)
{
	uint GroupCount = (InputCountBuffer[InputCountOffset] * Multiplier + Divisor - 1U) / Divisor;

	WriteDispatchIndirectArgs(IndirectDispatchArgsOut, 0, GroupCount, 1, 1);
}
