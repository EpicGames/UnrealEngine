// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VisualizeLightShape.usf: Shaders for visualizing light shapes
=============================================================================*/

#include "../Common.ush"
#include "../Hash.ush"
#include "../Visualization.ush"
#include "../MonteCarlo.ush"
#include "../SceneTexturesCommon.ush"
#include "../LightGridCommon.ush"

float3 SafeNormalize(float3 V)
{
	float L2 = dot(V, V);
	float InvLength = L2 > 1e-8 ? rsqrt(L2) : 0.0;
	return V * InvLength;
}

float3 GetLightShapeVertexTranslatedWorldPosition(float3 LocalPosition, FDeferredLightData LightData)
{
	float LightRadius = rcp(LightData.InvRadius);
	float3 OutPosition;

#if LIGHT_TYPE == LIGHT_TYPE_POINT
	OutPosition = LocalPosition * LightRadius + LightData.TranslatedWorldPosition;
#elif LIGHT_TYPE == LIGHT_TYPE_SPOT
	float ConeAngle = acos(LightData.SpotAngles.x) * LocalPosition.x;

	float SinConeAngle, CosConeAngle;
	sincos(ConeAngle, SinConeAngle, CosConeAngle);

	OutPosition = float3(CosConeAngle, LocalPosition.yz * SinConeAngle) * LightRadius;
	OutPosition = select(LocalPosition.xxx > 1.0001, 0.0, OutPosition);

	float3x3 LocalToWorldRotation = GetTangentBasis(-LightData.Direction);
	OutPosition = mul(OutPosition.zyx, LocalToWorldRotation) + LightData.TranslatedWorldPosition;
#else // LIGHT_TYPE_RECT
	float Width = LightData.SourceRadius * 2.0;
	float Height = LightData.SourceLength * 2.0;
	float CosBarnAngle = LightData.RectLightData.BarnCosAngle;
	float SinBarnAngle = sqrt(1.0 - CosBarnAngle * CosBarnAngle);
	float BarnLength = LightData.RectLightData.BarnLength;
	
	float3 LightForward = float3(1, 0, 0);
	float3 NormPos = LocalPosition * 2.0;
	float3 BaseRect = LocalPosition * float3(0, Width, Height);
	float3 BarnRect = BaseRect + float3(CosBarnAngle, SinBarnAngle * NormPos.yz) * BarnLength;

	// Project onto bounding sphere by intersecting the projection ray with the sphere
	// (i.e. substitute x^2 + y^2 + z^2 = r2 with o + dt and solve for t)
	float3 Dir = BarnRect + BaseRect;
	float A = dot(Dir, Dir);
	float B = -2.0 * dot(Dir, BaseRect);
	float C = dot(BaseRect, BaseRect) - LightRadius * LightRadius;
	float T = (-B + sqrt(B * B - 4.0 * A * C)) / (2.0 * A);
	float3 OuterRect = -BaseRect + Dir * T;

	// Rotate towards the intersection of light forward and the bounding sphere
	Dir = normalize(OuterRect);
	float AngleBetween = acos(dot(Dir, LightForward)) * NormPos.x;
	float SinAngleBetween, CosAngleBetween;
	sincos(AngleBetween, SinAngleBetween, CosAngleBetween);
	float3 RotationAxis = SafeNormalize(cross(Dir, LightForward));

	OutPosition = OuterRect * CosAngleBetween + cross(RotationAxis, OuterRect) * SinAngleBetween;
	OutPosition = select(NormPos.xxx < -0.0001, BarnRect, OutPosition);

	float3x3 LightToWorldRotation = float3x3(-LightData.Direction, cross(LightData.Tangent, -LightData.Direction), LightData.Tangent);
	OutPosition = mul(OutPosition, LightToWorldRotation) + LightData.TranslatedWorldPosition;
#endif

	return OutPosition;
}

struct FVSToPS
{
	uint LightSceneId : TEXCOORD0;
};

void VisualizeLightShapeVS(
	float3 LocalPosition : ATTRIBUTE0,
	uint2 ForwardLightIndex_LightType : ATTRIBUTE1,
	out FVSToPS VSToPS,
	out float4 OutPosition : SV_POSITION)
{
	// TODO: instanced stereo?
	ResolvedView = ResolveView();

	uint ForwardLightIndex = ForwardLightIndex_LightType.x;
	FForwardLightData ForwardLightData = GetForwardLightData(ForwardLightIndex, 0);
	FDeferredLightData LightData = ConvertToDeferredLight(ForwardLightData);

	float3 TranslatedWorldPosition = GetLightShapeVertexTranslatedWorldPosition(LocalPosition, LightData);

	VSToPS.LightSceneId = ForwardLightData.LightSceneId;
    OutPosition = mul(float4(TranslatedWorldPosition, 1.0), View.TranslatedWorldToClip);
}

float VisualizationToggleTime;
float InvFadeDuration;
float InvMaxDelayDistance;
float MaxDelay;
float MaxOpacity;

void VisualizeLightShapePS(
	FVSToPS VSToPS,
    float4 SvPosition : SV_POSITION,
    out float4 OutColor : SV_Target0)
{
	// TODO: instanced stereo?
	ResolvedView = ResolveView();

	float Random = Rand32ToFloat(XXHash32(VSToPS.LightSceneId));
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);
	float ViewDistance = length(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);
	
	// When visualization is just toggled on, delay fade in over distance so that lights far
	// away fade in first. When just toggled off, all lights fade out together
	bool bVisualize = VisualizationToggleTime > 0;
	float Delay = bVisualize ? saturate(1.0 - ViewDistance * InvMaxDelayDistance) * MaxDelay : 0.0;
	float Alpha = View.GameTime - Delay - abs(VisualizationToggleTime);
	Alpha = saturate(Alpha * InvFadeDuration);
	Alpha = bVisualize ? Alpha : (1.0 - Alpha);
	Alpha *= MaxOpacity;

	// Depth fade
	float4 ScreenPosition = SvPositionToScreenPosition(SvPosition);
	float PixelDepth = GetScreenPositionDepth(ScreenPosition);
	float OpaqueDepth = CalcSceneDepth(SvPositionToBufferUV(SvPosition));
	float DepthFadeDistance = 100.0;
	float DepthFadeAlpha = saturate((OpaqueDepth - PixelDepth) / DepthFadeDistance);
	Alpha *= DepthFadeAlpha;

	float3 Color = HSVtoRGB(float3(Random, 1, 1));

    OutColor = float4(Color, Alpha);
}
