// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VisualizeTexture.usf: VisualizeTexture shader
=============================================================================*/

#include "../Common.ush"

#define INPUT_MAPPING_COLOR 0
#define INPUT_MAPPING_DEPTH 1
#define INPUT_MAPPING_SHADOW 2

// to visualize the content of a texture to the screen, useful for debugging
// use "VisualizeTexture" in the console to activate
Texture2D VisualizeTexture2D;
SamplerState VisualizeTexture2DSampler;
Texture3D VisualizeTexture3D;
SamplerState VisualizeTexture3DSampler;  
TextureCube VisualizeTextureCube;
SamplerState VisualizeTextureCubeSampler;

//default Texture2D type is <float4>, which won't read the X28_G8UINT format properly.
Texture2D<uint4> VisualizeDepthStencil; 

Texture2D<uint> VisualizeUINT8Texture2D;

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
Texture2DMS<float4> VisualizeTexture2DMS;
TextureCubeArray VisualizeTextureCubeArray;
SamplerState VisualizeTextureCubeArraySampler;
#endif

// [0]: r:RGBAdd, g:SingleChannelMul, b:Add, a:FracScale
// [1]: r:BlinkState=0/1, g:Frac when 0, saturate when 1, b:array index, a:custom mip
// [2]: r:0=normal 1=depth, 2=shadow depth, g:unwrapcubemap, b:singlechannel(0-3), a:unused
float4 VisualizeParam[3];
// x, y (0 for 2D textures), z(only !=0 if 3d texture), w unused
float3 TextureExtent;


void VisualizeTexturePS(
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 UV = SvPosition.xy / TextureExtent.xy;

	// Longitude Latitude unwrapping
	float3 SpherePos;
	{
		float2 Angles = float2(2 * PI * (UV.x + 0.5f), PI * (1 - UV.y));
		float s = sin(Angles.y);
		SpherePos = float3(s * sin(Angles.x), -s * cos(Angles.x), -cos(Angles.y));
	}
	
	// Pixel based cube map unwrapping -- assumes 4x3 aspect ratio.  Produces a seamless panorama in the middle, connected to adjacent sky / floor faces.
	float3 CubePixelPos;
	{
		float2 LerpCoords = UV * float2(4.0, 3.0);
		float FaceCoords[3] =
		{
			frac(LerpCoords.x) * 2.0 - 1.0,
			frac(LerpCoords.y) * 2.0 - 1.0,
			1.0
		};
		
		// Table selecting a UV coordinate or "1" for each of the XYZ axes using a hex digit, preceded by a "0" or "2" to specify whether the axis should be negated.
		uint FaceUVMappingTable[12] =
		{
			0x010002,		// Positive Z faces (different rotations to line up with panoramic neighbor below)
			0x200102,
			0x212002,
			0x002102,
			0x020021,		// Positive X
			0x200221,		// Positive Y
			0x222021,		// Negative X
			0x002221,		// Negative Y
			0x210022,		// Negative Z faces (different rotations to line up with panoramic neighbor above)
			0x202122,
			0x012022,
			0x000122,
		};
		uint FaceUVMapping = FaceUVMappingTable[floor(LerpCoords.x) + floor(LerpCoords.y) * 4.0];
		CubePixelPos = float3(
			FaceCoords[(FaceUVMapping >> 16) & 0xf] * (1.0 - ((float) ((FaceUVMapping >> 20) & 0xf))),
			FaceCoords[(FaceUVMapping >>  8) & 0xf] * (1.0 - ((float) ((FaceUVMapping >> 12) & 0xf))),
			FaceCoords[(FaceUVMapping >>  0) & 0xf] * (1.0 - ((float) ((FaceUVMapping >>  4) & 0xf))));
	}

	// volume texture unwrapping
	float3 VolumePos;
	{
		uint LayerCount = TextureExtent.z;
		uint LayerUCount = (uint)sqrt(TextureExtent.z);
		uint LayerVCount = LayerCount / LayerUCount;

		uint LayerUId = (uint)(UV.x * LayerUCount);
		uint LayerVId = (uint)(UV.y * LayerVCount);
		uint LayerId = LayerUId + LayerUCount * LayerVId;

		VolumePos.z = LayerId / (TextureExtent.z - 1);
		VolumePos.x = frac(UV.x * LayerUCount);
		VolumePos.y = frac(UV.y * LayerVCount);
	}

	float MipLevel = VisualizeParam[1].a;
	float ArrayIndex = VisualizeParam[1].b;



#if TEXTURE_TYPE == 0
	// Select pixel accurate map coordinates if the visualization is 4x3 aspect ratio
	float4 TexLookup = TextureCubeSampleLevel(VisualizeTextureCube, VisualizeTextureCubeSampler, TextureExtent.x*0.75 == TextureExtent.y ? CubePixelPos : SpherePos, MipLevel);
#elif TEXTURE_TYPE == 2 || TEXTURE_TYPE == 6
	float4 TexLookup = Texture2DSampleLevel(VisualizeTexture2D, VisualizeTexture2DSampler, UV, MipLevel);
#elif TEXTURE_TYPE == 3
	#if FEATURE_LEVEL < FEATURE_LEVEL_SM4
		float4 TexLookup = float4(1.0f, 0.0f, 1.0f, 1.0f);
	#else
		float4 TexLookup = Texture3DSampleLevel(VisualizeTexture3D, VisualizeTexture3DSampler, VolumePos, MipLevel);
	#endif
#elif TEXTURE_TYPE == 4
	#if FEATURE_LEVEL < FEATURE_LEVEL_SM5
		float4 TexLookup = float4(1.0f, 0.0f, 1.0f, 1.0f);
	#else
		float4 TexLookup = TextureCubeArraySampleLevel(VisualizeTextureCubeArray, VisualizeTextureCubeArraySampler, SpherePos, ArrayIndex, MipLevel);
	#endif
#elif TEXTURE_TYPE == 5
	float4 TexLookup = float4(1,1,0,0);	// yellow, not yet supported
	
	#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
		TexLookup = VisualizeTexture2DMS.Load(int2(UV * TextureExtent.xy), 0);
	#endif
#elif TEXTURE_TYPE == 7
    float IntTexLookup = float(VisualizeUINT8Texture2D.Load(int3(UV * TextureExtent.xy, 0)));
	float4 TexLookup = IntTexLookup.xxxx;
#elif TEXTURE_TYPE == 8
	// 32 bits int, rather arbitrarily visualized as spread over the rgba channels
    int IntTexLookup = VisualizeUINT8Texture2D.Load(int3(UV * TextureExtent.xy, 0));
	float4 TexLookup;
	TexLookup.r = IntTexLookup & 0xFF;
	TexLookup.g = (IntTexLookup >> 8) & 0xFF;
	TexLookup.b = (IntTexLookup >> 16) & 0xFF;
	TexLookup.a = (IntTexLookup >> 24) & 0xFF;
#endif
	
	uint InputValueMapping = (uint)VisualizeParam[2].r;
	if(InputValueMapping != INPUT_MAPPING_COLOR)
	{
		float Depth = TexLookup.r;
		if (InputValueMapping == INPUT_MAPPING_DEPTH)
		{
			// current z buffer is negated (1 is near, 0 is far)
			Depth = 1 - Depth;

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
			// map depth to nice colors
			float Scale = 64.0; // This 64 looks like a hardware limit.
			Depth = (exp2(Depth * Scale) - 1.0) * (1.0 / (exp2(Scale) - 1.0));
			Depth *= Depth;
			Depth *= Depth;

			float Value = Depth * Depth;
			TexLookup = float4(pow(Value, 0.25f), pow(Value, 1), pow(Value, 4), 0);
		} 
		else if (InputValueMapping == INPUT_MAPPING_SHADOW)
		{
			// linearly blend from White (near) to Teal (far).
			float3 TealBlue = float3(54.f/255.f, 117.f/255.f, 136.f/255.f);
			float3 White = float3(1.0f, 1.0f, 1.0f);
			TexLookup.rgb = lerp(White, TealBlue, Depth);
		}
#if TEXTURE_TYPE == 6
		// inside #ifdef to avoid D3DDebug error:
		//    The resource return type for component 0 declared in the shader code (UINT) is not compatible with the Shader Resource View format bound to slot 1 of the Pixel Shader unit (UNORM).  This mismatch is invalid if the shader actually uses the view (e.g. it is not skipped due to shader code branching). [ EXECUTION ERROR #361: DEVICE_DRAW_RESOURCE_RETURN_TYPE_MISMATCH]
		float fStencil = VisualizeDepthStencil.Load(int3(UV * TextureExtent.xy, 0)).g;
		fStencil /= 255.0f;
		TexLookup.a = fStencil;
#endif

#else
		}
		TexLookup.r = Depth;
#endif
	}

	float4 RGBAMask = 0;
	int SingleChannel = (int)VisualizeParam[2].z;
	switch (SingleChannel)
	{
		case 0: RGBAMask = float4(1,0,0,0); break;
		case 1: RGBAMask = float4(0,1,0,0); break;
		case 2: RGBAMask = float4(0,0,1,0); break;
		case 3: RGBAMask = float4(0,0,0,1); break;
		default: RGBAMask = float4(0,0,0,0); break;
	}
	float Add = dot( TexLookup, RGBAMask ) * VisualizeParam[0].y + VisualizeParam[0].z;


	OutColor.rgb = TexLookup.rgb * VisualizeParam[0].xxx + Add.xxx;

	float3 ScaledColor = OutColor.rgb * VisualizeParam[0].w;
	float3 FracOutColor = lerp(ScaledColor, saturate(ScaledColor), VisualizeParam[1].y);

	float3 AlternateColor = FracOutColor;
	
	// blink red if <0
	if(OutColor.r < 0.0f || OutColor.g < 0.0f || OutColor.b < 0.0f)
	{
		AlternateColor = float3(1, 0, 0);
	}
	
	// blink blue if not finite
#if FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	if (isfinite(OutColor.rgb).x == false)
	{
		AlternateColor = float3(0, 0, 1);
	}
#endif
	OutColor.rgb = lerp(FracOutColor, AlternateColor, VisualizeParam[1].x);	
	OutColor.a = 0;
}