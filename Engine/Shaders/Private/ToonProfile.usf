// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush" 
#include "ToonProfileCommon.ush"


#ifdef ToonProfileUpdateAtlasTileCS

int2 ToonProfileTextureResolution;
uint ToonProfileIndex;

float ToonProfileShadowExtinctionCoefficientInInvCentimeter;

float DiffuseRampOffsetStrength;
float DiffuseRampOffsetSize;

float SpecularRampOffsetStrength;
float SpecularRampOffsetSize;

float ShadowHatchingPatternSize;
float ShadowHatchingPatternStrength;

float DiffuseIndirectScale;
float SpecularIndirectScale;
float bDiffuseRampIncludeShadow;
float SpecularIndirectRampRepetition;

RWTexture2DArray<float4>	ToonProfileTexture;

Texture2D<float4>			DiffuseRampOffsetTexture;
SamplerState				DiffuseRampOffsetTextureSampler;

Texture2D<float4>			SpecularRampOffsetTexture;
SamplerState				SpecularRampOffsetTextureSampler;

Texture2D<float4>			ShadowHatchingPatternTexture;
SamplerState				ShadowHatchingPatternTextureSampler;

Buffer<float4>				DiffuseRampBufferSRV;
Buffer<float4>				SpecularRampBufferSRV;
Buffer<float4>				SpecularIndirectRampBufferSRV;
Buffer<float4>				ShadowRampBufferSRV;

[numthreads(8, 8, 1)]
void ToonProfileUpdateAtlasTileCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const float4 DefaultColor = float4(1, 0, 1, 0);

	const float ProfileResolution = ToonProfileTextureResolution.x;
	const float ProfileResolutionInv = 1.0f / ProfileResolution;

	const uint2 PixelPos = DispatchThreadId.xy;
	if (all(PixelPos < uint2(ToonProfileTextureResolution)))
	{
		if (PixelPos.y == TOON_PROFILE_DIFFUSE_RAMP_Y)
		{
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = DiffuseRampBufferSRV[PixelPos.x];
		}
		else if (PixelPos.y == TOON_PROFILE_SPECULAR_RAMP_Y)
		{
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = SpecularRampBufferSRV[PixelPos.x];
		}
		else if (PixelPos.y == TOON_PROFILE_SPEC_IND_RAMP_Y)
		{
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = SpecularIndirectRampBufferSRV[PixelPos.x];
		}
		else if (PixelPos.y == TOON_PROFILE_SHADOW_RAMP_Y)
		{
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = ShadowRampBufferSRV[PixelPos.x];
		}
		else if (PixelPos.y == TOON_PROFILE_PARAMETERS_Y)
		{
			if (PixelPos.x == TOON_PROFILE_EVAL_PARAMS_X)
			{
				ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = float4(DiffuseRampOffsetStrength, DiffuseRampOffsetSize, SpecularRampOffsetStrength, SpecularRampOffsetSize);
			}
			else if (PixelPos.x == TOON_PROFILE_EVAL_PARAMS2_X)
			{
				ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = float4(bDiffuseRampIncludeShadow, 0.0f, 0.0f, 0.0f);
			}
			else if (PixelPos.x == TOON_PROFILE_SHADOW_PARAMS_X)
			{
				ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = float4(ToonProfileShadowExtinctionCoefficientInInvCentimeter, ShadowHatchingPatternSize, ShadowHatchingPatternStrength, 0.0f);
			}
			else if (PixelPos.x == TOON_PROFILE_GI_PARAMS_X)
			{
				ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = float4(DiffuseIndirectScale, SpecularIndirectScale, SpecularIndirectRampRepetition, 0.0f);
			}
			else
			{
				ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = DefaultColor;
			}
		}
		else if (PixelPos.y >= uint(TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX * ProfileResolution) && PixelPos.y < uint(TOON_PROFILE_TEXTURES_Y + (TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX +1) * ProfileResolution))
		{
		/*
			For a Toon profile of full resolution N, we render an edge of 1 texel all around to make sure the texture is repeatable, marked `e` below.
			So the actual bake resolution of the dithering pattern is N-2.
			And The actual sampled texture is the square defined by the `X` below.

		0,0	--------------------------------- 1,0
			| e | e | e | e | e | e | e | e |
			----X-----------------------X----
			| e |	|	|	|	|	|	| e |
			---------------------------------
			| e |	|	|	|	|	|	| e |
			---------------------------------
			| e |	|	|	|	|	|	| e |
			---------------------------------
			| e |	|	|	|	|	|	| e |
			---------------------------------
			| e |	|	|	|	|	|	| e |
			---------------------------------
			| e |	|	|	|	|	|	| e |
			----X-----------------------X----
			| e | e | e | e | e | e | e | e |
		0,1 --------------------------------- 1,1
		*/
			const float2 FullOrigin = float2(0.0f, TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX * ProfileResolution);
			const float2 BakeOrigin = FullOrigin + 1.0f;

			const float  BakeResolution = ProfileResolution - 2.0f;
			const float  BakeResolutionInv = 1.0f / BakeResolution;

			const float2 FullCoord  = float2(PixelPos.xy) + 0.5f.xx;
			const float2 FullUVs    = (FullCoord - BakeOrigin) * BakeResolutionInv;

			const float DiffuseRampOffsetTextureValue  = DiffuseRampOffsetTexture.SampleLevel (DiffuseRampOffsetTextureSampler,  FullUVs, 0.0f).x;
			const float SpecularRampOffsetTextureValue = SpecularRampOffsetTexture.SampleLevel(SpecularRampOffsetTextureSampler, FullUVs, 0.0f).y;
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = float4(DiffuseRampOffsetTextureValue, SpecularRampOffsetTextureValue, 0.0f, 0.0f);
		}
		else if (PixelPos.y >= uint(TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_SHADOWHATCHING_IDX * ProfileResolution) && PixelPos.y < uint(TOON_PROFILE_TEXTURES_Y + (TOON_PROFILE_TEXTURE_SHADOWHATCHING_IDX + 1) * ProfileResolution))
		{
			// This texture is stored with an edge for filtering similarly to TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX
			const float2 FullOrigin = float2(0.0f, TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_SHADOWHATCHING_IDX * ProfileResolution);
			const float2 BakeOrigin = FullOrigin + 1.0f;

			const float  BakeResolution = ProfileResolution - 2.0f;
			const float  BakeResolutionInv = 1.0f / BakeResolution;

			const float2 FullCoord  = float2(PixelPos.xy) + 0.5f.xx;
			const float2 FullUVs    = (FullCoord - BakeOrigin) * BakeResolutionInv;

			const float4 ShadowHatchingPatternValues = ShadowHatchingPatternTexture.SampleLevel(ShadowHatchingPatternTextureSampler, FullUVs, 0.0f).rgba;
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = ShadowHatchingPatternValues;
		}
		else
		{
			ToonProfileTexture[uint3(PixelPos, ToonProfileIndex)] = DefaultColor;
		}
	}
}

#endif
