// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Shared/ToonProfileDefinitions.h"

#if SUPPORTS_INDEPENDENT_SAMPLERS
#define SharedToonProfileSampler	View.SharedBilinearClampedSampler
#else
#define SharedToonProfileSampler	View.ToonProfileSampler
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

float3 EvaluateToonProfileRamp(in float ToonProfileId, in float NoL, in float TOON_PROFILE_RAMP)
{
	const float3 ToonProfileCoord = float3(
		NoL,																	// The toon ramp is applied over NoL in [-1, 1]
		(TOON_PROFILE_RAMP + 0.5f) * View.ToonProfileTextureSizeAndInvSize.w,	// Center of the row of texel
		ToonProfileId															// Slice index
	);
	return View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).xyz;
}

// NoL input should be in [0,1], remapped from [-1, 1].
float3 EvaluateToonProfileDiffuseRamp(in float ToonProfileId, in float NoL)
{
	return EvaluateToonProfileRamp(ToonProfileId, saturate(NoL), TOON_PROFILE_DIFFUSE_RAMP_Y);
}

float3 EvaluateToonProfileSpecularRamp(in float ToonProfileId, in float Specular)
{
	return EvaluateToonProfileRamp(ToonProfileId, Specular, TOON_PROFILE_SPECULAR_RAMP_Y);
}

float EvaluateToonProfileSpecularIndirectRamp(in float ToonProfileId, in float Level)
{
	return EvaluateToonProfileRamp(ToonProfileId, Level, TOON_PROFILE_SPEC_IND_RAMP_Y).x;
}

float EvaluateToonProfileShadowHatchingPatternDistributionRamp(in float ToonProfileId, in float Transmittance)
{
	return EvaluateToonProfileRamp(ToonProfileId, Transmittance, TOON_PROFILE_SHADOW_RAMP_Y).r;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FToonProfileEvaluateSettings
{
	float DiffuseRampOffsetStrength;
	float DiffuseRampOffsetSize;
	float SpecularRampOffsetStrength;
	float SpecularRampOffsetSize;
	float bDiffuseRampIncludeShadow;
};

FToonProfileEvaluateSettings GetToonProfileEvaluateSettings(in float ToonProfileId)
{
	const float3 ToonProfileCoord = float3(
		(TOON_PROFILE_EVAL_PARAMS_X + 0.5f) * View.ToonProfileTextureSizeAndInvSize.z,
		(TOON_PROFILE_PARAMETERS_Y  + 0.5f) * View.ToonProfileTextureSizeAndInvSize.w,		// Center of the row of texel
		ToonProfileId																		// EvaluateToonProfileDiffuseDither / Slice index
	);
	float4 Data = View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0);

	FToonProfileEvaluateSettings Ret;
	Ret.DiffuseRampOffsetStrength	= Data.x;
	Ret.DiffuseRampOffsetSize		= Data.y;
	Ret.SpecularRampOffsetStrength	= Data.z;
	Ret.SpecularRampOffsetSize		= Data.w;

	const float3 ToonProfileCoord2 = float3(
		(TOON_PROFILE_EVAL_PARAMS2_X + 0.5f) * View.ToonProfileTextureSizeAndInvSize.z,
		ToonProfileCoord.yz);
	float4 Data2 = View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord2, 0).xyzw;

	Ret.bDiffuseRampIncludeShadow = Data2.x;

	return Ret;
}

float EvaluateToonProfileDiffuseDither(in float ToonProfileId, in float2 PatternUVs, in FToonProfileEvaluateSettings ToonProfileEvaluateSettings)
{
	if (ToonProfileEvaluateSettings.DiffuseRampOffsetStrength > 0.0f)
	{
		// See ToonProfile.usf for the baking/sampling description.

		PatternUVs = frac(PatternUVs * ToonProfileEvaluateSettings.DiffuseRampOffsetSize);

		const float2 FullOrigin = float2(0.0f, TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX * View.ToonProfileDitherPatternSizeAndInvSize.x);
		const float2 BakeOrigin = FullOrigin + 1.0f;
		
		const float2 BakeCoords        = BakeOrigin + PatternUVs * View.ToonProfileDitherPatternSizeAndInvSize.x;

		const float3 ToonProfileCoord = float3(
			BakeCoords.xy * View.ToonProfileTextureSizeAndInvSize.zw,
			ToonProfileId
		);

		return ToonProfileEvaluateSettings.DiffuseRampOffsetStrength * (View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).x * 2.0f - 1.0f);
	}
	return 0.0f;
}

float EvaluateToonProfileSpecularDither(in float ToonProfileId, in float2 PatternUVs, in FToonProfileEvaluateSettings ToonProfileEvaluateSettings)
{
	if (ToonProfileEvaluateSettings.SpecularRampOffsetStrength > 0.0f)
	{
		// See ToonProfile.usf for the baking/sampling description.

		PatternUVs = frac(PatternUVs * ToonProfileEvaluateSettings.SpecularRampOffsetSize);

		const float2 FullOrigin = float2(0.0f, TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX * View.ToonProfileTextureSizeAndInvSize.x);
		const float2 BakeOrigin = FullOrigin + 1.0f;
		
		const float2 BakeCoords = BakeOrigin + PatternUVs * View.ToonProfileDitherPatternSizeAndInvSize.x;

		const float3 ToonProfileCoord = float3(
			BakeCoords.xy * View.ToonProfileTextureSizeAndInvSize.zw,
			ToonProfileId
		);

		return ToonProfileEvaluateSettings.SpecularRampOffsetStrength * (View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).y * 2.0f - 1.0f);
	}
	return 0.0f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FToonProfileShadowSettings
{
	float ShadowExtinctionCoefficientInInvCentimeter;	// Needed because transmittance evaluation code from ShadowProjectionPixelShader.usf uses centimeters.
	float ShadowMFPInMeters;							// Needed for storage in substrate Subsurface Header.
	float ShadowHatchingPatternSize;
	float ShadowHatchingPatternStrength;
};

FToonProfileShadowSettings GetToonProfileShadowSettings(in float ToonProfileId)
{
	const float3 ToonProfileCoord = float3(
		(TOON_PROFILE_SHADOW_PARAMS_X + 0.5f) * View.ToonProfileTextureSizeAndInvSize.z,
		(TOON_PROFILE_PARAMETERS_Y    + 0.5f) * View.ToonProfileTextureSizeAndInvSize.w,	// Center of the row of texel
		ToonProfileId																		// Slice index
	);
	float4 Data = View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).xyzw;

	FToonProfileShadowSettings Ret;
	Ret.ShadowExtinctionCoefficientInInvCentimeter = Data.x;

	const float ShadowExtinctionCoefficientInInvMeter = Ret.ShadowExtinctionCoefficientInInvCentimeter * (1.0f / CENTIMETER_TO_METER); // 1/cm => 1/m
	Ret.ShadowMFPInMeters = 1.0f / max(0.000001f, ShadowExtinctionCoefficientInInvMeter);

	Ret.ShadowHatchingPatternSize = Data.y;

	Ret.ShadowHatchingPatternStrength = Data.z;

	return Ret;
}

// SUBSTRATE_TODO instead of shadow/transmittance, it could also account for NoL or AO?
float EvaluateToonProfileShadowHatching(in float ToonProfileId, in float2 PatternUVs, in FToonProfileShadowSettings ToonProfileShadowSettings, in float Transmittance)
{
	if (ToonProfileShadowSettings.ShadowHatchingPatternStrength > 0.0f)
	{
		// See ToonProfile.usf for the baking/sampling description.

		PatternUVs = frac(PatternUVs * ToonProfileShadowSettings.ShadowHatchingPatternSize);

		const float2 FullOrigin = float2(0.0f, TOON_PROFILE_TEXTURES_Y + TOON_PROFILE_TEXTURE_SHADOWHATCHING_IDX * View.ToonProfileTextureSizeAndInvSize.x);
		const float2 BakeOrigin = FullOrigin + 1.0f;

		const float2 BakeCoords = BakeOrigin + PatternUVs * View.ToonProfileDitherPatternSizeAndInvSize.x;

		const float3 ToonProfileCoord = float3(
			BakeCoords.xy * View.ToonProfileTextureSizeAndInvSize.zw,
			ToonProfileId
		);

		const float4 ShadowHatching = View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).rgba;

		const float Distribution = EvaluateToonProfileShadowHatchingPatternDistributionRamp(ToonProfileId, Transmittance).x;

		// SUBSTRATE_TODO optimise
		const float Step = 0.25f;
		float FinalShadow = 0.0;
		if (Distribution <= 0.25)
		{
			FinalShadow = lerp(ShadowHatching.a, ShadowHatching.b, saturate((Distribution - 0.00) / 0.25));
		}
		else if (Distribution <= 0.5)
		{
			FinalShadow = lerp(ShadowHatching.b, ShadowHatching.g, saturate((Distribution - 0.25) / 0.25));
		}
		else if (Distribution <= 0.75)
		{
			FinalShadow = lerp(ShadowHatching.g, ShadowHatching.r, saturate((Distribution - 0.50) / 0.25));
		}
		else if (Distribution <= 1.0)
		{
			FinalShadow = lerp(ShadowHatching.r, 1.0f            , saturate((Distribution - 0.75) / 0.25));
		}

		const float FadeToWhite = ToonProfileShadowSettings.ShadowHatchingPatternStrength;
		return lerp(1.0f, FinalShadow, FadeToWhite);
	}
	return Transmittance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FToonProfileGISettings
{
	float DiffuseIndirectScale;
	float SpecularIndirectScale;
	float SpecularIndirectRampRepetition;
};

FToonProfileGISettings GetToonProfileGISettings(in float ToonProfileId)
{
	const float3 ToonProfileCoord = float3(
		(TOON_PROFILE_GI_PARAMS_X  + 0.5f) * View.ToonProfileTextureSizeAndInvSize.z,
		(TOON_PROFILE_PARAMETERS_Y + 0.5f) * View.ToonProfileTextureSizeAndInvSize.w,		// Center of the row of texel
		ToonProfileId																		// Slice index
	);
	float4 Data = View.ToonProfileTexture.SampleLevel(SharedToonProfileSampler, ToonProfileCoord, 0).xyzw;

	FToonProfileGISettings Ret;
	Ret.DiffuseIndirectScale = Data.x;
	Ret.SpecularIndirectScale = Data.y;
	Ret.SpecularIndirectRampRepetition = Data.z;
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#if SUBSTRATE_EXPERIMENTAL_TOON_REFLECTION_QUANTIZATION

float3 SubstrateToonQuantize(
	in float ToonProfileId,
	in float3 InSpecularLighting)
{
	if (ToonProfileId >= 0.0f)
	{
		FToonProfileGISettings ToonProfileGISettings = GetToonProfileGISettings(ToonProfileId);
		const float RampScale = ToonProfileGISettings.SpecularIndirectRampRepetition;
		if (RampScale <= 0.0)
		{
			return InSpecularLighting;
		}

		float3 SpecularLighting = log(max(0.001, InSpecularLighting));
		float3 SpecularLightingHSV = LinearRGB_2_HSV(SpecularLighting);

		SpecularLightingHSV.z *= RampScale;

		const float RampSpecLevel = frac(SpecularLightingHSV.z);
		const float UntouchedSpec = SpecularLightingHSV.z - RampSpecLevel;

		SpecularLightingHSV.z = UntouchedSpec + EvaluateToonProfileSpecularIndirectRamp(ToonProfileId, RampSpecLevel);

		SpecularLightingHSV.z *= (1.0f / RampScale);

		SpecularLighting = HSV_2_LinearRGB(SpecularLightingHSV);
		SpecularLighting = exp(SpecularLighting);
		return SpecularLighting;
	}
	return InSpecularLighting;
}

#endif
