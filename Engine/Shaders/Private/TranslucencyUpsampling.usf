// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TranslucencyUpsampling.usf: PostProcessing shader to upscale
=============================================================================*/

#include "Common.ush"
#include "ScreenPass.ush"


uint2 StencilPixelPosMin;
uint2 StencilPixelPosMax;
FScreenTransform SvPositionToStencilPixelCoord;
uint StencilMask;
Texture2D<uint2> StencilTexture;

void UpsampleResponsiveAAPS(FScreenVertexOutput Input, out float4 OutColor : SV_Target0)
{
	float2 SvPosition = Input.Position.xy;

	uint2 StencilPixelPos = uint2(ApplyScreenTransform(SvPosition, SvPositionToStencilPixelCoord));
	StencilPixelPos = clamp(StencilPixelPos, StencilPixelPosMin, StencilPixelPosMax);

	uint StencilValue = StencilTexture.Load(int3(StencilPixelPos, 0)) STENCIL_COMPONENT_SWIZZLE;

	// If the responsive AA stencil was not set, discard the pixel to not write the responsive AA in the full res depth buffer.
	if ((StencilValue & StencilMask) == 0)
	{
		discard;
	}

	OutColor = float(0.0).xxxx;
}
