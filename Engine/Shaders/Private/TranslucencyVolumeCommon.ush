// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define SharedAmbientInnerSampler View.SharedBilinearClampedSampler
#define SharedAmbientOuterSampler View.SharedBilinearClampedSampler
#define SharedDirectionalInnerSampler View.SharedBilinearClampedSampler
#define SharedDirectionalOuterSampler View.SharedBilinearClampedSampler
#define SharedSpecularDirectionInnerSampler View.SharedBilinearClampedSampler
#define SharedSpecularDirectionOuterSampler View.SharedBilinearClampedSampler

void ComputeVolumeUVs(float3 TranslatedWorldPosition, float3 LightingPositionOffset, out float3 InnerVolumeUVs, out float3 OuterVolumeUVs, out float FinalLerpFactor)
{
	// Apply an offset to the world position used for lighting, which breaks up artifacts from using a low res volume texture
	InnerVolumeUVs = (TranslatedWorldPosition + LightingPositionOffset * View.TranslucencyLightingVolumeInvSize[0].w - View.TranslucencyLightingVolumeMin[0].xyz) * View.TranslucencyLightingVolumeInvSize[0].xyz;
	OuterVolumeUVs = (TranslatedWorldPosition + LightingPositionOffset * View.TranslucencyLightingVolumeInvSize[1].w - View.TranslucencyLightingVolumeMin[1].xyz) * View.TranslucencyLightingVolumeInvSize[1].xyz;

	// Controls how fast the lerp between the inner and outer cascades happens
	// Larger values result in a shorter transition distance
	float TransitionScale = 6;
	// Setup a 3d lerp factor going to 0 at the edge of the inner volume
	float3 LerpFactors = saturate((.5f - abs(InnerVolumeUVs - .5f)) * TransitionScale);
	FinalLerpFactor = LerpFactors.x * LerpFactors.y * LerpFactors.z;
}

float4 GetAmbientLightingVectorFromTranslucentLightingVolume(float3 InnerVolumeUVs, float3 OuterVolumeUVs, float FinalLerpFactor)
{
	// Lookup the inner and outer cascade ambient lighting values
	float4 InnerLighting = Texture3DSampleLevel(TranslucencyLightingVolumeAmbientInner, SharedAmbientInnerSampler, InnerVolumeUVs, 0);
	float4 OuterLighting = Texture3DSampleLevel(TranslucencyLightingVolumeAmbientOuter, SharedAmbientOuterSampler, OuterVolumeUVs, 0);

	// Lerp between cascades
	return lerp(OuterLighting, InnerLighting, FinalLerpFactor);
}

float3 GetDirectionalLightingVectorFromTranslucentLightingVolume(float3 InnerVolumeUVs, float3 OuterVolumeUVs, float FinalLerpFactor)
{
	// Fetch both the ambient and directional values for both cascades
	float3 InnerVector1 = Texture3DSampleLevel(TranslucencyLightingVolumeDirectionalInner, SharedDirectionalInnerSampler, InnerVolumeUVs, 0).rgb;
	float3 OuterVector1 = Texture3DSampleLevel(TranslucencyLightingVolumeDirectionalOuter, SharedDirectionalOuterSampler, OuterVolumeUVs, 0).rgb;

	// Lerp between cascades
	return lerp(OuterVector1, InnerVector1, FinalLerpFactor);
}

float3 GetSpecularDirectionFromTranslucentLightingVolume(float3 InnerVolumeUVs, float3 OuterVolumeUVs, float FinalLerpFactor)
{
	// Sample the specular direction from the dedicated volume with proper cascade lerp
	float3 InnerDir = Texture3DSampleLevel(TranslucencySpecularDirectionInner, SharedSpecularDirectionInnerSampler, InnerVolumeUVs, 0).rgb * 2.0f - 1.0f;
	float3 OuterDir = Texture3DSampleLevel(TranslucencySpecularDirectionOuter, SharedSpecularDirectionOuterSampler, OuterVolumeUVs, 0).rgb * 2.0f - 1.0f;
	
	// Lerp between cascades
	float3 BlendedDir = lerp(OuterDir, InnerDir, FinalLerpFactor);
	
	float BlendedDirLength = length(BlendedDir);
	return BlendedDirLength > 0 ? BlendedDir / BlendedDirLength : float3(0, 0, 0);
}

#include "SHCommon.ush"

FTwoBandSHVectorRGB ReconstructSHCoefficients(float4 AmbientLightingVector, float3 DirectionalLightingVector)
{
	// Reconstruct the SH coefficients based on what was encoded
	FTwoBandSHVectorRGB TranslucentLighting;
	TranslucentLighting.R.V.x = AmbientLightingVector.r;
	TranslucentLighting.G.V.x = AmbientLightingVector.g;
	TranslucentLighting.B.V.x = AmbientLightingVector.b;
	float3 NormalizedAmbientColor = AmbientLightingVector.rgb / (Luminance(AmbientLightingVector.rgb) + 0.00001f);

	// Scale the monocrome directional coefficients with the normalzed ambient color as an approximation to the uncompressed values
	TranslucentLighting.R.V.yzw = DirectionalLightingVector.rgb * NormalizedAmbientColor.r;
	TranslucentLighting.G.V.yzw = DirectionalLightingVector.rgb * NormalizedAmbientColor.g;
	TranslucentLighting.B.V.yzw = DirectionalLightingVector.rgb * NormalizedAmbientColor.b;

	return TranslucentLighting;
}

void GetVolumeLightingNonDirectional(float4 AmbientLightingVector, float3 DiffuseColor, inout float3 InterpolatedLighting, out float4 VolumeLighting)
{
	// Normal is not taken into account with non directional lighting, and only the ambient term of the SH coefficients are needed
	FOneBandSHVectorRGB TranslucentLighting;
	TranslucentLighting.R.V.x = AmbientLightingVector.r;
	TranslucentLighting.G.V.x = AmbientLightingVector.g;
	TranslucentLighting.B.V.x = AmbientLightingVector.b;

	FOneBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH1(1);
	VolumeLighting = float4(DotSH1(TranslucentLighting, DiffuseTransferSH), AmbientLightingVector.a);
	InterpolatedLighting = DiffuseColor * VolumeLighting.rgb;
}

void GetVolumeLightingDirectional(FTwoBandSHVectorRGB TranslucentLighting, float DirectionalLightContribution, float3 WorldNormal, float3 DiffuseColor, float DirectionalLightingIntensity, inout float3 InterpolatedLighting, out float4 VolumeLighting)
{
	// Compute diffuse lighting which takes the normal into account
	FTwoBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH(WorldNormal * DirectionalLightingIntensity, 1);
	VolumeLighting = float4(max(half3(0, 0, 0), DotSH(TranslucentLighting, DiffuseTransferSH)), DirectionalLightContribution);
	InterpolatedLighting += DiffuseColor * VolumeLighting.rgb;
}

void GetVolumeLightingDirectional(float4 AmbientLightingVector, float3 DirectionalLightingVector, float3 WorldNormal, float3 DiffuseColor, float DirectionalLightingIntensity, inout float3 InterpolatedLighting, out float4 VolumeLighting)
{
	FTwoBandSHVectorRGB TranslucentLighting = ReconstructSHCoefficients(AmbientLightingVector, DirectionalLightingVector);
	
	GetVolumeLightingDirectional(TranslucentLighting, AmbientLightingVector.a, WorldNormal, DiffuseColor, DirectionalLightingIntensity, InterpolatedLighting, VolumeLighting);
}

float4 GetVolumeLightingDirectional(float4 AmbientLightingVector, float3 DirectionalLightingVector, float3 WorldNormal, float DirectionalLightingIntensity)
{
	float3 DiffuseColor = 1.0f;
	float3 InterpolatedLighting = 0.0f;
	float4 VolumeLighting = 0.0f;

	GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, WorldNormal, DiffuseColor, DirectionalLightingIntensity, InterpolatedLighting, VolumeLighting);

	return VolumeLighting;
}
