// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * TranslucencyVolumeInjectionCommon.ush: Common functions for calculating lighting in a volume to use on translucency
 */

#pragma once

#include "Common.ush"
#include "SHCommon.ush"

float3 GetTranslatedWorldPositionForLighting(float3 TranslatedWorldPosition, float3 NormalizedLightVector, uint VolumeCascadeIndex)
{
	float VoxelSize = View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
	// Push out the position for lighting in the oposite direction of the L vector 
	// to make sure we always have half a voxel distance from the light to avoid overblown lighting intensities.
	float3 LightingOffset = -NormalizedLightVector * 0.5f * VoxelSize;
	return TranslatedWorldPosition + LightingOffset;
}

float GetBorderLerpFactorFromVolumeUVs(float3 VolumeUVs, uint VolumeCascadeIndex)
{
	// Larger values result in a shorter transition distance
	float TransitionScale = 10;

	// Rescale the UVs to make the fade go to 0 before the edge of the volume
	float3 FadeUVs = VolumeUVs * (1 + 4 * View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].w) - 2 * View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].w;
	// Setup a 3d lerp factor going to 0 near the edge of the outer volume
	float3 LerpFactors = saturate((.5f - abs(FadeUVs - .5f)) * TransitionScale);
	float FinalLerpFactor = LerpFactors.x * LerpFactors.y * LerpFactors.z;

	return FinalLerpFactor;
}

float GetBorderLerpFactor(float2 UV, uint LayerIndex, uint VolumeCascadeIndex)
{
	const float3 VolumeUVs = float3(UV, (LayerIndex + .5f) * View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].w);
	return GetBorderLerpFactorFromVolumeUVs(VolumeUVs, VolumeCascadeIndex);
}

float GetBorderLerpFactor(uint3 VolumeCoord, uint VolumeCascadeIndex)
{
	const float3 VolumeUVs = (VolumeCoord + 0.5f) * View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].w;
	return GetBorderLerpFactorFromVolumeUVs(VolumeUVs, VolumeCascadeIndex);
}

void AccumulateSHLighting(
	float3 NormalizedLightVector,
	float3 Lighting,
	float DirectionalLightContribution,
	inout float4 OutColor0,
	inout float4 OutColor1)
{
	FTwoBandSHVectorRGB SHLighting = MulSH(SHBasisFunction(NormalizedLightVector), Lighting);

	// Directional light contribution in w
	OutColor0 += float4(SHLighting.R.V.x, SHLighting.G.V.x, SHLighting.B.V.x, DirectionalLightContribution);

	float3 LuminanceWeights = LuminanceFactors();
	float3 Coefficient0 = float3(SHLighting.R.V.y, SHLighting.G.V.y, SHLighting.B.V.y);
	float3 Coefficient1 = float3(SHLighting.R.V.z, SHLighting.G.V.z, SHLighting.B.V.z);
	float3 Coefficient2 = float3(SHLighting.R.V.w, SHLighting.G.V.w, SHLighting.B.V.w);
	OutColor1 += float4(dot(Coefficient0, LuminanceWeights), dot(Coefficient1, LuminanceWeights), dot(Coefficient2, LuminanceWeights), 0);
}
