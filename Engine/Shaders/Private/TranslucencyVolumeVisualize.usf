// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * TranslucentLightInjectionShaders.usf: Shaders for calculating lighting in a volume to use on translucency
 */

#include "Common.ush"
#include "SHCommon.ush"

Texture3D TranslucencyLightingVolumeAmbientInner;
Texture3D TranslucencyLightingVolumeAmbientOuter;
Texture3D TranslucencyLightingVolumeDirectionalInner;
Texture3D TranslucencyLightingVolumeDirectionalOuter;
Texture3D TranslucencySpecularDirectionInner;
Texture3D TranslucencySpecularDirectionOuter;

#include "TranslucencyVolumeCommon.ush"

Texture3D TranslucencyLightingVolumeAmbient;
Texture3D TranslucencyLightingVolumeDirectional;

uint VolumeCascadeIndex;
uint VolumeSize;

float ProbeRadiusScale;

float3 GetTranslatedWorldPosition(float2 UV, uint LayerIndex)
{
	float ZPosition = View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].z + (LayerIndex + .5f) * View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
	float3 TranslatedWorldPosition = float3(View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].xy + UV / View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].xy, ZPosition);
	return TranslatedWorldPosition;
}

float3 GetTranslatedWorldPosition(uint3 VolumeCoord)
{
	return View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].xyz + (VolumeCoord + .5f) * View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
}

struct FVisualizeTranslucencyVolumeVSToPS
{
	nointerpolation float3 ProbeCoord : TEXCOORD0;
	nointerpolation float4 CellSphere : TEXCOORD1;
	float3 PositionTWS : TEXCOORD2;
};

void VisualizeTranslucencyVolumeVS(
	uint VertexId : SV_VertexID,
	uint InstanceId : SV_InstanceID,
	out FVisualizeTranslucencyVolumeVSToPS Output,
	out float4 OutPosition : SV_POSITION
)
{
	float ProbeRadius = ProbeRadiusScale * (VolumeCascadeIndex + 1);

	float3 LocalPosition;
	LocalPosition.x = VertexId & 0x1 ? 1.0f : -1.0f;
	LocalPosition.y = VertexId & 0x2 ? 1.0f : -1.0f;
	LocalPosition.z = VertexId & 0x4 ? 1.0f : -1.0f;
	LocalPosition *= ProbeRadius;

	uint3 ProbeCoord;
	ProbeCoord.x = InstanceId % VolumeSize;
	ProbeCoord.y = (InstanceId / VolumeSize) % VolumeSize;
	ProbeCoord.z = InstanceId / (VolumeSize * VolumeSize);

	float3 ProbeTranslatedWorldCenter = GetTranslatedWorldPosition(ProbeCoord);

	Output.CellSphere.xyz = ProbeTranslatedWorldCenter;
	Output.CellSphere.w = ProbeRadius;
	Output.PositionTWS = LocalPosition + ProbeTranslatedWorldCenter;
	Output.ProbeCoord = ProbeCoord;
	OutPosition = mul(float4(Output.PositionTWS, 1.0f), PrimaryView.TranslatedWorldToClip);
}

void VisualizeTranslucencyVolumePS(
	FVisualizeTranslucencyVolumeVSToPS Input,
	out float4 OutColor : SV_Target0
)
{
	float3 RayDirection = Input.PositionTWS;
	float2 SphereIntersections = RayIntersectSphere(0, RayDirection, Input.CellSphere);
	float3 IntersectionPosition = SphereIntersections.x * RayDirection;
	clip(SphereIntersections.x);

	float3 SphereNormal = normalize(IntersectionPosition - Input.CellSphere.xyz);

	float3 Lighting = 0.0f;

	float4 AmbientLightingVector = TranslucencyLightingVolumeAmbient[Input.ProbeCoord];
	float3 DirectionalLightingVector = TranslucencyLightingVolumeDirectional[Input.ProbeCoord].rgb;

	Lighting += GetVolumeLightingDirectional(AmbientLightingVector, DirectionalLightingVector, SphereNormal, /*DirectionalLightingIntensity*/ 1.0f).rgb;

	Lighting *= View.PreExposure;

	OutColor = float4(Lighting, 1.0f);
}
