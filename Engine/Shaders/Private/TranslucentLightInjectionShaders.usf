// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * TranslucentLightInjectionShaders.usf: Shaders for calculating lighting in a volume to use on translucency
 */

#include "Common.ush"
#include "ComputeShaderUtils.ush"

#define TREAT_MAXDEPTH_UNSHADOWED 1

#if INJECTION_PIXEL_SHADER
	#include "/Engine/Generated/Material.ush"
	#include "VolumeLightingCommon.ush"
#endif

#include "TranslucencyVolumeInjectionCommon.ush"
#include "DynamicLightingCommon.ush"

#define SUPPORT_CONTACT_SHADOWS 0
#include "DeferredLightingCommon.ush"

uint VolumeCascadeIndex;
float4 SimpleLightPositionAndRadius;
float4 SimpleLightColorAndExponent;

#if RADIAL_ATTENUATION==0
#define USE_CLOUD_TRANSMITTANCE 1
#include "VolumetricCloudCommon.ush"
uint VolumetricCloudShadowEnabled;

#include "/Engine/Private/SkyAtmosphereCommon.ush"
uint AtmospherePerPixelTransmittanceEnabled;
#endif

#if VIRTUAL_SHADOW_MAP
#include "VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#ifndef ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#define ADAPTIVE_VOLUMETRIC_SHADOW_MAP 0
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
#include "HeterogeneousVolumes/HeterogeneousVolumesAdaptiveVolumetricShadowMapSampling.ush"
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

// To use DEBUG_ONE_VOXEL, one must enable ShaderPrint from TranslucentLighting.cpp.
#define DEBUG_ONE_VOXEL 0
#if DEBUG_ONE_VOXEL
#include "/Engine/Private/ShaderPrint.ush"
#endif

float3 GetTranslatedWorldPosition(float2 UV, uint LayerIndex)
{
	float ZPosition = View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].z + (LayerIndex + .5f) * View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
	float3 TranslatedWorldPosition = float3(View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].xy + UV / View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].xy, ZPosition);
	return TranslatedWorldPosition;
}

float3 GetTranslatedWorldPosition(uint3 VolumeCoord)
{
	return View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].xyz + (VolumeCoord + .5f) * View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
}

float CalcSimpleLightAttenuation(float3 TranslatedWorldPosition)
{
	float3 WorldLightVector = SimpleLightPositionAndRadius.xyz - TranslatedWorldPosition;

	float Attenuation = 1;

	if (SimpleLightColorAndExponent.w == 0)
	{
		float DistanceSqr = dot( WorldLightVector, WorldLightVector );

		// Sphere falloff (technically just 1/d2 but this avoids inf)
		Attenuation = 1 / ( DistanceSqr + 1 );
	
		float LightRadiusMask = Square(saturate(1 - Square(DistanceSqr / (SimpleLightPositionAndRadius.w * SimpleLightPositionAndRadius.w))));
		Attenuation *= LightRadiusMask;
	}
	else
	{
		Attenuation = RadialAttenuation(WorldLightVector / SimpleLightPositionAndRadius.w, SimpleLightColorAndExponent.w);
	}

	return Attenuation;
}

/** Pixel shader that calculates direct lighting for a simple light (unshadowed point light) for all the affected voxels of a volume texture. */
void SimpleLightInjectMainPS(
	FWriteToSliceGeometryOutput Input,
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1
	)
{
	OutColor0 = 0;
	OutColor1 = 0;

	float3 TranslatedWorldPosition = GetTranslatedWorldPosition(Input.Vertex.UV, Input.LayerIndex);
	float3 NormalizedLightVector = normalize(SimpleLightPositionAndRadius.xyz - TranslatedWorldPosition);
	float3 TranslatedWorldPositionForLighting = GetTranslatedWorldPositionForLighting(TranslatedWorldPosition, NormalizedLightVector, VolumeCascadeIndex);

	float Attenuation = CalcSimpleLightAttenuation(TranslatedWorldPositionForLighting);
	float3 Lighting = SimpleLightColorAndExponent.rgb / PI * Attenuation;

	float DirectionalLightContribution = 0.0f;
	AccumulateSHLighting(NormalizedLightVector, Lighting, DirectionalLightContribution,
		OutColor0, OutColor1);
}

float GetTranslucencyVolumeLocalLightAttenuation(float3 TranslatedWorldPosition, FDeferredLightData LightData)
{
	float3 ToLight, L;
	float Attenuation = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

	if (LightData.bRectLight)
	{
		if (Attenuation > 0.0f)
		{
			// Use a more accurate rect light approximation
			FRect Rect = GetRect(ToLight, LightData);
			Attenuation *= IntegrateLight(Rect);
		}
	}
	else if (LightData.bInverseSquared)
	{
		// This path still uses the legacy local light attenuation, so add that back in
		float DistanceSqr = dot(ToLight, ToLight);
		Attenuation *= 1.0f / (DistanceSqr + 1.0f);
	}

	return Attenuation;
}

#if INJECTION_PIXEL_SHADER

#include "LightDataUniforms.ush"
#include "LightFunctionCommon.ush"

// WorldSpace planes to clip the cascade for ShadoewMethod1
float4 ClippingPlanes[2];
/** 1 if the light is a spotlight, 0 otherwise. */
float SpotlightMask;

float GetLightFunctionShadowFactor(float3 TranslatedWorldPositionForLighting)
{
	float ShadowFactor = 1;

	// Apply light function after edge fading, so that a black light function at the edges can cause distant translucency to also be black
	#if APPLY_LIGHT_FUNCTION
		float4 LightVector = mul(float4(TranslatedWorldPositionForLighting, 1),LightFunctionTranslatedWorldToLight);
		LightVector.xyz /= LightVector.w;
		float3 LightFunction = GetLightFunctionColor(LightVector.xyz, TranslatedWorldPositionForLighting);

		// We only suport monochrome light functions
		ShadowFactor = dot(LightFunction, .3333f).x;
	#endif

	return ShadowFactor;
}

int VirtualShadowMapId;

/** Pixel shader that calculates direct lighting for all the affected voxels of a volume texture. */
void InjectMainPS(
	FWriteToSliceGeometryOutput Input,
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1
	)
{
	ResolvedView = ResolveView();

	OutColor0 = 0;
	OutColor1 = 0;

	float3 TranslatedWorldPosition = GetTranslatedWorldPosition(Input.Vertex.UV, Input.LayerIndex);
	
	FDeferredLightData LightData = InitDeferredLightFromUniforms();
	float3 LightVector = LightData.Direction;

	// 0: no contribution, 1:full contribution
	float Masking = 1.0f;

#if DEBUG_ONE_VOXEL
	const float DebugVoxelSize = View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
	const float DebugHalfVoxelSize = DebugVoxelSize * 0.5f;
	const float3 WorldPosition = (TranslatedWorldPosition - DFDemote(PrimaryView.PreViewTranslation));
	const int3 IntegerPos = (int3)(WorldPosition / (DebugVoxelSize));

	// Enable shader print for a single voxel containing VoxelIncludedWorldPosition for cascade 0.
	const float3 VoxelIncludedWorldPosition = float3(0.5, 0.5, 0.5);
	FShaderPrintContext Context = InitShaderPrintContext(all((WorldPosition - DebugHalfVoxelSize) <= VoxelIncludedWorldPosition) && all((WorldPosition + DebugHalfVoxelSize) > VoxelIncludedWorldPosition) && VolumeCascadeIndex == 0, uint2(200, 200));
	AddAABBTWS(Context, TranslatedWorldPosition - DebugHalfVoxelSize, TranslatedWorldPosition + DebugHalfVoxelSize, float4(1, 1, 1, 1));
	AddSphereTWS(Context, TranslatedWorldPosition, 2.0f, float4(0, 1, 0, 1));
#endif
	
	#if RADIAL_ATTENUATION 
	{
		// cull voxels outside the light radius (value is < 0)
		LightVector = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
		clip(1.0f / (LightData.InvRadius * LightData.InvRadius) - dot(LightVector, LightVector));
	}
	#else
	{
		// directional light
		float DistToNear = -dot(ClippingPlanes[0], float4(TranslatedWorldPosition, 1));
		float DistToFar = -dot(ClippingPlanes[1], float4(TranslatedWorldPosition, 1));
		
		// cull volumes outside the cascade (value is < 0)
		clip(DistToNear);
		clip(DistToFar);

		// fade cascade transition regions (additively blended so it does a cross fade)
		Masking *= saturate(DistToNear * ShadowInjectParams.x);
		Masking *= saturate(DistToFar * ShadowInjectParams.y);
	}
	#endif

	float3 NormalizedLightVector = normalize(LightVector);
	float3 TranslatedWorldPositionForLighting = GetTranslatedWorldPositionForLighting(TranslatedWorldPosition, NormalizedLightVector, VolumeCascadeIndex);

#if DEBUG_ONE_VOXEL
	AddSphereTWS(Context, TranslatedWorldPositionForLighting, 5.0f, float4(0, 1, 1, 1));
#endif

	{
		float Attenuation = 1.0f;
		float ShadowFactor = 1.0f;

#if RADIAL_ATTENUATION
		Attenuation *= GetTranslucencyVolumeLocalLightAttenuation(TranslatedWorldPositionForLighting, LightData);
#endif

		{
			bool bUnused = false;
			ShadowFactor *= ComputeVolumeShadowing(
				TranslatedWorldPositionForLighting,
				// Maintain the legacy logic of "spotlight" vs "everything else"
				!LightData.bSpotLight,
				LightData.bSpotLight,
				bUnused);
		}

#if VIRTUAL_SHADOW_MAP
		FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapTranslatedWorld(VirtualShadowMapId, TranslatedWorldPositionForLighting);
		ShadowFactor *= VirtualShadowMapSample.ShadowFactor;
#endif

#if ADAPTIVE_VOLUMETRIC_SHADOW_MAP
		ShadowFactor *= AVSM_SampleTransmittance(TranslatedWorldPositionForLighting, LightData.TranslatedWorldPosition);
#endif // ADAPTIVE_VOLUMETRIC_SHADOW_MAP

		// Apply light function (it will also fade out at the edge of the volume so overall dimming of light using light function is not a recommended workflow)
		ShadowFactor *= GetLightFunctionShadowFactor(TranslatedWorldPositionForLighting);

		#if RADIAL_ATTENUATION==0
		// Apply cloud shadow for atmosphere directional lights only if needed
		if (VolumetricCloudShadowEnabled > 0)
		{
			float OutOpticalDepth = 0.0f;
			ShadowFactor *= lerp(1.0f, GetCloudVolumetricShadow(TranslatedWorldPositionForLighting, CloudShadowmapTranslatedWorldToLightClipMatrix, CloudShadowmapFarDepthKm, CloudShadowmapTexture, CloudShadowmapSampler, OutOpticalDepth), CloudShadowmapStrength);
		}
		#endif
		
		if (VolumeCascadeIndex == 1)
		{
			float FinalLerpFactor = GetBorderLerpFactor(Input.Vertex.UV, Input.LayerIndex, VolumeCascadeIndex);

			#if RADIAL_ATTENUATION 
			// For local lights, fade attenuation to 0 for the border voxels
			Attenuation = lerp(0, Attenuation, FinalLerpFactor);
			ShadowFactor = lerp(0.0f, ShadowFactor, FinalLerpFactor);
			#else
			// Fade out shadowing for the border voxels
			// The border voxels are used to light all translucency outside of both lighting volumes
			ShadowFactor = lerp(1.0f, ShadowFactor, FinalLerpFactor);
			#endif
		}

		float3 Lighting = LightData.Color / PI * Attenuation * ShadowFactor;

		#if RADIAL_ATTENUATION==0
		// Apply color atmosphere transmittance for atmosphere directional lights only if ndeeded
		if (AtmospherePerPixelTransmittanceEnabled > 0)
		{
			const float3 TranslatedWorldPlanetCenterToPos = (TranslatedWorldPositionForLighting - View.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;
			Lighting *= GetAtmosphereTransmittance(TranslatedWorldPlanetCenterToPos, NormalizedLightVector, View.SkyAtmosphereBottomRadiusKm, View.SkyAtmosphereTopRadiusKm, View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);
		}
		#endif

		float DirectionalLightContribution = 0;
#if !RADIAL_ATTENUATION 
		DirectionalLightContribution = Attenuation * ShadowFactor;
#endif
		AccumulateSHLighting(NormalizedLightVector, Lighting, DirectionalLightContribution,
			OutColor0, OutColor1);
	}

	// debug, make inner cascase green
//	if(VolumeCascadeIndex == 0) OutColor0 = float4(0,1,0,1);

	OutColor0 *= Masking;
	OutColor1 *= Masking;
}

#endif // #if INJECTION_PIXEL_SHADER

#ifdef InjectBatchMainCS

#include "LightGridCommon.ush"
#include "LightData.ush"

#include "LightFunctionAtlas/LightFunctionAtlasCommon.usf"

//int VirtualShadowMapId;
StructuredBuffer<uint> BatchedLocalLights;
uint MaxBatchedLocalLights;

uint3 VolumeSize;

RWTexture3D<float4> RWTranslucencyLightingVolumeAmbient;
RWTexture3D<float4> RWTranslucencyLightingVolumeDirectional;

#if !USE_UAV_TYPED_LOAD
Texture3D<float4> TranslucencyLightingVolumeAmbient;
Texture3D<float4> TranslucencyLightingVolumeDirectional;
#endif

#if INDIRECT_VOXEL_DISPATCH
StructuredBuffer<uint> VoxelAllocator;
StructuredBuffer<uint> VoxelData;
#endif

/** Compute shader that calculates direct lighting for all the affected voxels of a volume texture. */
[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void InjectBatchMainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupId : SV_GroupID,
	uint GroupIndex : SV_GroupIndex)
{
#if INDIRECT_VOXEL_DISPATCH
	const uint VoxelIndex = GetUnWrappedDispatchThreadId(GroupId, GroupIndex, THREADGROUP_SIZE_X);

	if (VoxelIndex >= VoxelAllocator[0])
	{
		return;
	}

	const uint PackedVolumeCoord = VoxelData[VoxelIndex];
	const uint3 VolumeCoord = uint3(PackedVolumeCoord & 0x3FF, (PackedVolumeCoord >> 10) & 0x3FF, (PackedVolumeCoord >> 20) & 0x3FF);
#else
	const uint3 VolumeCoord = DispatchThreadId.xyz;

	if (any(VolumeCoord >= VolumeSize))
	{
		return;
	}
#endif

	float4 OutColor0 = 0;
	float4 OutColor1 = 0;

	float3 TranslatedWorldPosition = GetTranslatedWorldPosition(VolumeCoord);
	
	const uint NumUints = (MaxBatchedLocalLights + 31U) / 32U;
	for (uint UintIndex = 0; UintIndex < NumUints; ++UintIndex)
	{
		uint CurrentMask = BatchedLocalLights[UintIndex];
		while (CurrentMask != 0)
		{
			int BitIndex = firstbitlow(CurrentMask);
			CurrentMask = CurrentMask & ~(1U << BitIndex);
			uint LocalLightIndex = 32U * UintIndex + BitIndex;
			
			{
				FLocalLightData LocalLightData = GetLocalLightData(LocalLightIndex, 0);
				FDeferredLightData LightData = ConvertToDeferredLight(LocalLightData);

				float3 LightVector = LightData.TranslatedWorldPosition - TranslatedWorldPosition;
				if (rcp(Square(LightData.InvRadius)) > dot(LightVector, LightVector))
				{
					float3 NormalizedLightVector = normalize(LightVector);
					float3 TranslatedWorldPositionForLighting = GetTranslatedWorldPositionForLighting(TranslatedWorldPosition, NormalizedLightVector, VolumeCascadeIndex);

					float Attenuation = GetTranslucencyVolumeLocalLightAttenuation(TranslatedWorldPositionForLighting, LightData);
					float ShadowFactor = 1.0f;

					if (Attenuation > 0.0f)
					{
#if VIRTUAL_SHADOW_MAP
						if (LocalLightData.Internal.VirtualShadowMapId != INDEX_NONE)
						{
							FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapTranslatedWorld(LocalLightData.Internal.VirtualShadowMapId, TranslatedWorldPositionForLighting);
							ShadowFactor *= VirtualShadowMapSample.ShadowFactor;
						}
#endif

						if (VolumeCascadeIndex == 1)
						{
							float FinalLerpFactor = GetBorderLerpFactor(VolumeCoord, VolumeCascadeIndex);
							// For local lights, fade attenuation to 0 for the border voxels
							Attenuation = lerp(0, Attenuation, FinalLerpFactor);
							ShadowFactor = lerp(0.0f, ShadowFactor, FinalLerpFactor);
						}

						FLightFunctionColor LightFunctionColor = 1.0f;
					#if	USE_LIGHT_FUNCTION_ATLAS
						LightFunctionColor = GetLocalLightFunctionCommon(TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
					#endif

						float3 Lighting = LightFunctionColor * LightData.Color / PI * Attenuation * ShadowFactor;

						float DirectionalLightContribution = 0;
						AccumulateSHLighting(NormalizedLightVector, Lighting, DirectionalLightContribution,
							OutColor0, OutColor1);
					}
				}
			}
		}
	}

#if USE_UAV_TYPED_LOAD
	RWTranslucencyLightingVolumeAmbient[VolumeCoord] = RWTranslucencyLightingVolumeAmbient[VolumeCoord] + OutColor0;
	RWTranslucencyLightingVolumeDirectional[VolumeCoord] = RWTranslucencyLightingVolumeDirectional[VolumeCoord] + OutColor1;
#else
	RWTranslucencyLightingVolumeAmbient[VolumeCoord] = TranslucencyLightingVolumeAmbient[VolumeCoord] + OutColor0;
	RWTranslucencyLightingVolumeDirectional[VolumeCoord] = TranslucencyLightingVolumeDirectional[VolumeCoord] + OutColor1;
#endif
}

#endif // InjectBatchMainCS

#ifdef ClearTranslucentLightingVolumeCS

RWTexture3D<float4> RWAmbient0;
RWTexture3D<float4> RWDirectional0;
RWTexture3D<float4> RWAmbient1;
RWTexture3D<float4> RWDirectional1;

[numthreads(CLEAR_BLOCK_SIZE, CLEAR_BLOCK_SIZE, CLEAR_BLOCK_SIZE)]
void ClearTranslucentLightingVolumeCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{

	uint3 Index = uint3(GroupId.x * CLEAR_BLOCK_SIZE + GroupThreadId.x, GroupId.y * CLEAR_BLOCK_SIZE + GroupThreadId.y, GroupId.z * CLEAR_BLOCK_SIZE + GroupThreadId.z);
	RWAmbient0[Index.xyz] = float4(0.0, 0.0, 0.0, 0.0);
	RWDirectional0[Index.xyz] = float4(0.0, 0.0, 0.0, 0.0);
	RWAmbient1[Index.xyz] = float4(0.0, 0.0, 0.0, 0.0);
	RWDirectional1[Index.xyz] = float4(0.0, 0.0, 0.0, 0.0);
}
#endif // ClearTranslucentLightingVolumeCS