// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * TranslucentLightingShaders.usf: Shaders for calculating lighting in a volume to use on translucency
 */

#include "Common.ush"
#include "SceneTexturesCommon.ush"
#include "BlueNoise.ush"
#include "ComputeShaderUtils.ush"
#include "TranslucencyVolumeInjectionCommon.ush"

struct FWriteToSliceVertexOutput
{
	FScreenVertexOutput Vertex;
#if USING_VERTEX_SHADER_LAYER
	uint LayerIndex : SV_RenderTargetArrayIndex;
#else
	uint LayerIndex : TEXCOORD1;
#endif
};

/** Z index of the minimum slice in the range. */
int MinZ;

float4 UVScaleBias;
uint VolumeCascadeIndex;

/** Vertex shader that writes to a range of slices of a volume texture. */
void WriteToSliceMainVS(
	float2 InPosition : ATTRIBUTE0,
	float2 InUV       : ATTRIBUTE1,
	uint LayerIndex : SV_InstanceID,
	out FWriteToSliceVertexOutput Output
	)
{
	Output.Vertex.Position = float4( InPosition, 0, 1 );
	// Remap UVs based on the subregion of the volume texture being rendered to
    Output.Vertex.UV = InUV * UVScaleBias.xy + UVScaleBias.zw;
	Output.LayerIndex = LayerIndex + MinZ;
}

/** Encodes HDR linear scene color for storage in the 8 bit light attenuation texture. */
MaterialFloat3 EncodeSceneColorForMaterialNode(MaterialFloat3 LinearSceneColor)
{
	// Preserving a range from [0, 10]
	// Remap values to get more bits of precision in the darks
	return pow(LinearSceneColor * .1f, .25f);
}

Texture2D SceneColorTexture;
#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define SceneColorTextureSampler GlobalPointClampedSampler
#else
	SamplerState SceneColorTextureSampler;
#endif

void CopySceneColorMain(FScreenVertexOutput Input, out float4 OutColor : SV_Target0)
{
	float4 LinearColor = Texture2DSample(SceneColorTexture, SceneColorTextureSampler, Input.UV);
#if SCENE_COLOR_COPY_ENCODED
	OutColor = float4(EncodeSceneColorForMaterialNode(LinearColor.rgb), LinearColor.a);
#else
	OutColor = LinearColor;
#endif
}

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5

/** Geometry shader that writes to a range of slices of a volume texture. */
[maxvertexcount(3)]
void WriteToSliceMainGS(triangle FWriteToSliceVertexOutput Input[3], inout TriangleStream<FWriteToSliceGeometryOutput> OutStream)
{
	FWriteToSliceGeometryOutput Vertex0;
	Vertex0.Vertex = Input[0].Vertex;
	Vertex0.LayerIndex = Input[0].LayerIndex;

	FWriteToSliceGeometryOutput Vertex1;
	Vertex1.Vertex = Input[1].Vertex;
	Vertex1.LayerIndex = Input[1].LayerIndex;

	FWriteToSliceGeometryOutput Vertex2;
	Vertex2.Vertex = Input[2].Vertex;
	Vertex2.LayerIndex = Input[2].LayerIndex;

	OutStream.Append(Vertex0);
	OutStream.Append(Vertex1);
	OutStream.Append(Vertex2);
}

float3 GetTranslatedWorldPositionFromVolumeCoord(uint3 VolumeCoord, uint CascadeIndex)
{
	return View.TranslucencyLightingVolumeMin[CascadeIndex].xyz + (VolumeCoord + .5f) * View.TranslucencyLightingVolumeInvSize[CascadeIndex].w;
}

float3 GetTranslatedWorldPositionFromVolumeCoord(uint3 VolumeCoord)
{
	return View.TranslucencyLightingVolumeMin[VolumeCascadeIndex].xyz + (VolumeCoord + .5f) * View.TranslucencyLightingVolumeInvSize[VolumeCascadeIndex].w;
}

#ifdef DilateMarkedVoxelsCS

RWTexture3D<uint> RWVolumeMarkTexture;
Texture3D<uint> VolumeMarkTexture;
uint3 VolumeSize;

void MarkVoxel(int3 VoxelCoord)
{
	if (all(VoxelCoord >= 0) && all(VoxelCoord < (int3)VolumeSize))
	{
		RWVolumeMarkTexture[VoxelCoord] = 255;
	}
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void DilateMarkedVoxelsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint3 VolumeCoord = DispatchThreadId.xyz;

	if (any(VolumeCoord >= VolumeSize))
	{
		return;
	}
	
	if (VolumeMarkTexture[VolumeCoord] == 0)
	{
		return;
	}

	MarkVoxel(VolumeCoord + int3(0, 0, 0));
	MarkVoxel(VolumeCoord + int3(1, 1, 1));
	MarkVoxel(VolumeCoord + int3(-1, 1, 1));
	MarkVoxel(VolumeCoord + int3(1, -1, 1));
	MarkVoxel(VolumeCoord + int3(-1, -1, 1));
	MarkVoxel(VolumeCoord + int3(1, 1, -1));
	MarkVoxel(VolumeCoord + int3(-1, 1, -1));
	MarkVoxel(VolumeCoord + int3(1, -1, -1));
	MarkVoxel(VolumeCoord + int3(-1, -1, -1));
}

#endif // DilateMarkedVoxelsCS

#ifdef GatherMarkedVoxelsCS

Texture3D<uint> VolumeMarkTexture;
uint3 VolumeSize;

RWStructuredBuffer<uint> RWVoxelAllocator;
RWStructuredBuffer<uint> RWVoxelData;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void GatherMarkedVoxelsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint3 VolumeCoord = DispatchThreadId.xyz;

	if (any(VolumeCoord >= VolumeSize))
	{
		return;
	}
	
	if (VolumeMarkTexture[VolumeCoord] == 0)
	{
		return;
	}

	uint PackedVoxelIndex;
	InterlockedAdd(RWVoxelAllocator[0], 1, PackedVoxelIndex);
	RWVoxelData[PackedVoxelIndex] = (VolumeCoord.x) | (VolumeCoord.y << 10) | (VolumeCoord.z << 20);
}

#endif // GatherMarkedVoxelsCS

#ifdef InitIndirectArgsCS

RWBuffer<uint> RWIndirectArgs;
StructuredBuffer<uint> VoxelAllocator;

[numthreads(1, 1, 1)]
void InitIndirectArgsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Need to span the launch size across X,Y,Z so any single dimension does not exceed the 64k
	// limit as per https://microsoft.github.io/DirectX-Specs/d3d/MeshShader.html#dispatchmesh-api
	// The product of X*Y*Z can be 2^22, so we convert 1D->3D->1D to work around this limitation.
	const uint DimensionLimit = (1u << 16u) - 1u;

	WriteDispatchIndirectArgs(RWIndirectArgs, 0, GetGroupCountWrapped(DivideAndRoundUp(VoxelAllocator[0], THREADGROUP_SIZE), DimensionLimit.xx));
}

#endif

#ifdef FilterTranslucentVolumeCS

/** Filter pass ouputs. */
RWTexture3D<float4> RWTranslucencyLightingVolumeAmbient;
RWTexture3D<float4> RWTranslucencyLightingVolumeDirectional;
RWTexture3D<float4> RWTranslucencySpecularDirection;

/** Filter pass inputs. */
Texture3D TranslucencyLightingVolumeAmbient;
SamplerState TranslucencyLightingVolumeAmbientSampler;
Texture3D TranslucencyLightingVolumeDirectional;
SamplerState TranslucencyLightingVolumeDirectionalSampler;
Texture3D<uint> TranslucencyLightingVolumeMarkTexture;

Texture3D HistoryAmbient;
Texture3D HistoryDirectional;

SamplerState HistoryAmbientSampler;
SamplerState HistoryDirectionalSampler;

Texture3D<uint> HistoryMark;

Texture3D HistorySpecularDirection;

// MegaLights injection
Texture3D<float4> MegaLightsAmbient;
Texture3D<float4> MegaLightsDirectional;

uint3 VolumeSize;
float TexelSize;

float4 PrevTranslucencyLightingVolumeMin;
float4 PrevTranslucencyLightingVolumeInvSize;

float3 HistoryTextureBilinearUVMin;
float3 HistoryTextureBilinearUVMax;

float HistoryWeight;

uint OutputSpecularDirection;
float SpecularDirectionHistoryWeight;

uint HasSpecularDirectionHistory;
uint HasMarkHistory;

/** Filters the volume lighting to reduce aliasing. */
[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void FilterTranslucentVolumeCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint3 VolumeCoord = DispatchThreadId.xyz;

	if (any(VolumeCoord >= VolumeSize))
	{
		return;
	}

	#if USE_MARK_TEXTURE
	if (TranslucencyLightingVolumeMarkTexture[VolumeCoord] == 0)
	{
		// Skip empty voxels
		RWTranslucencyLightingVolumeAmbient[VolumeCoord] = 0.0f;
		RWTranslucencyLightingVolumeDirectional[VolumeCoord] = 0.0f;
		if (OutputSpecularDirection)
		{
			RWTranslucencySpecularDirection[VolumeCoord] = 0.5f;
		}
		return;
	}
	#endif

	float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromVolumeCoord(VolumeCoord);
	const float3 VolumeUV = (VolumeCoord + 0.5f) * TexelSize;

	float4 TextureValue0 = 0.0f;
	float4 TextureValue1 = 0.0f;
	float InvWeight = 1.0f;

	// Temporal filter state - declared outside USE_TEMPORAL_FILTER scope so specular direction filtering can access them
	int3 HistoryCoord = (TranslatedWorldPosition - PrevTranslucencyLightingVolumeMin.xyz) / PrevTranslucencyLightingVolumeInvSize.w;
	float HistoryAlpha = 1.0f;

	if (any(HistoryCoord < 0) || any(HistoryCoord >= VolumeSize))
	{
		HistoryAlpha = 0;
	}

	#if USE_MARK_TEXTURE
	if (HasMarkHistory && HistoryMark[HistoryCoord] == 0)
	{
		// don't use history if it was not computed
		HistoryAlpha = 0;
	}
	#endif

	if (VolumeCascadeIndex == 1)
	{
		// fade out at border to prevent issues with usage of clamped sampler during shading passes
		HistoryAlpha *= GetBorderLerpFactorFromVolumeUVs(VolumeUV, VolumeCascadeIndex);
	}

	#if !USE_SPATIAL_FILTER
	{
		TextureValue0 = TranslucencyLightingVolumeAmbient[VolumeCoord];
		TextureValue1 = TranslucencyLightingVolumeDirectional[VolumeCoord];

		#if INJECT_MEGALIGHTS
		{
			TextureValue0 += MegaLightsAmbient[VolumeCoord];
			TextureValue1 += MegaLightsDirectional[VolumeCoord];
		}
		#endif
	}
	#endif

	#if USE_TEMPORAL_FILTER
	{
		float4 HistoryTextureValue0 = HistoryAmbient[HistoryCoord];
		float4 HistoryTextureValue1 = HistoryDirectional[HistoryCoord];

		TextureValue0 = lerp(TextureValue0, HistoryTextureValue0, HistoryWeight * HistoryAlpha);
		TextureValue1 = lerp(TextureValue1, HistoryTextureValue1, HistoryWeight * HistoryAlpha);
	}
	#elif USE_SPATIAL_FILTER
	{
		TextureValue0 = 0.0f;
		TextureValue1 = 0.0f;

		// Use trilinear filtering to filter neighbors to the current voxel with minimal texture fetches
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(1, 1, 1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(-1, 1, 1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(1, -1, 1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(-1, -1, 1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(1, 1, -1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(-1, 1, -1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(1, -1, -1));
		TextureValue0 += Texture3DSample(TranslucencyLightingVolumeAmbient, TranslucencyLightingVolumeAmbientSampler, VolumeUV + .5f * TexelSize * float3(-1, -1, -1));

		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(1, 1, 1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(-1, 1, 1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(1, -1, 1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(-1, -1, 1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(1, 1, -1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(-1, 1, -1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(1, -1, -1));
		TextureValue1 += Texture3DSample(TranslucencyLightingVolumeDirectional, TranslucencyLightingVolumeDirectionalSampler, VolumeUV + .5f * TexelSize * float3(-1, -1, -1));

		InvWeight = 1.0f / 8;
	}
	#endif
	
	RWTranslucencyLightingVolumeAmbient[VolumeCoord] = TextureValue0 * InvWeight;
	RWTranslucencyLightingVolumeDirectional[VolumeCoord] = TextureValue1 * InvWeight;

	if (OutputSpecularDirection)
	{
		// Compute specular direction from filtered SH band 1 coefficients
		float3 FilteredSH1 = TextureValue1.rgb * InvWeight;
		float FilteredSH1Length = length(FilteredSH1);
		float3 SpecularDir = FilteredSH1Length > 0 ? FilteredSH1 / FilteredSH1Length : 0;
		SpecularDir = float3(-SpecularDir.z, -SpecularDir.x, SpecularDir.y);

		if (HasSpecularDirectionHistory)
		{
			// Temporal filter the specular direction independently
			float3 HistoryDir = HistorySpecularDirection[HistoryCoord].rgb * 2.0f - 1.0f;
			float3 BlendedDir = lerp(SpecularDir, HistoryDir, SpecularDirectionHistoryWeight * HistoryAlpha); // TODO: Use slerp instead?
			float BlendedDirLength = length(BlendedDir);
			SpecularDir = BlendedDirLength > 0 ? BlendedDir / BlendedDirLength : 0;
		}

		RWTranslucencySpecularDirection[VolumeCoord] = float4(SpecularDir * 0.5f + 0.5f, 0);
	}
}

#endif // FilterTranslucentVolumeCS

#include "CubemapCommon.ush"

/** Add AmbientCubemap color to the volume. */
void InjectAmbientCubemapMainPS(
	FWriteToSliceGeometryOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	// can be optimized by moving it into the vertex/geometry shader
	OutColor = float4(ComputeAmbientCubemapAvgColor(), 0);
}


#endif // FEATURE_LEVEL


#ifdef FrustumCullVoxelsCS

#define MAX_CAMERA_PLANES (5)

RWStructuredBuffer<uint> RWVoxelAllocator0;
RWStructuredBuffer<uint> RWVoxelAllocator1;

float4 CameraFrustumPlanes[MAX_CAMERA_PLANES];

uint NumVoxels;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void FrustumCullVoxelsCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	const float3 VoxelCenter0 = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 0);
	const float3 VoxelExtent0 = View.TranslucencyLightingVolumeInvSize[0].w / 2;

	bool bNearFrustum0 = true;
	for (uint PlaneIndex = 0; PlaneIndex < MAX_CAMERA_PLANES; ++PlaneIndex)
	{
		float DistanceFromVoxelCenterToPlane = dot(CameraFrustumPlanes[PlaneIndex], float4(VoxelCenter0, 1.0f));
		float Radius = dot(VoxelExtent0, abs(CameraFrustumPlanes[PlaneIndex].xyz));
		bNearFrustum0 = bNearFrustum0 && (DistanceFromVoxelCenterToPlane <= Radius);
	}

	const float3 VoxelCenter1 = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 1);
	const float3 VoxelExtent1 = View.TranslucencyLightingVolumeInvSize[1].w / 2;

	bool bNearFrustum1 = true;
	for (uint PlaneIndex = 0; PlaneIndex < MAX_CAMERA_PLANES; ++PlaneIndex)
	{
		float DistanceFromVoxelCenterToPlane = dot(CameraFrustumPlanes[PlaneIndex], float4(VoxelCenter1, 1.0f));
		float Radius = dot(VoxelExtent1, abs(CameraFrustumPlanes[PlaneIndex].xyz));
		bNearFrustum1 = bNearFrustum1 && (DistanceFromVoxelCenterToPlane <= Radius);
	}

	if (bNearFrustum0)
	{
		InterlockedAdd(RWVoxelAllocator0[0], 1);
	}

	if (bNearFrustum1)
	{
		InterlockedAdd(RWVoxelAllocator1[0], 1);
	}
}
#endif // FrustumCullVoxelsCS


#ifdef DebugTranslucencyLightingVolumeCS

#include "/Engine/Private/ShaderPrint.ush"

Texture3D<uint> VolumeMarkTexture0;
Texture3D<uint> VolumeMarkTexture1;

StructuredBuffer<uint> VoxelAllocator0;
StructuredBuffer<uint> VoxelAllocator1;

StructuredBuffer<uint> FrustumCullVoxelAllocator0;
StructuredBuffer<uint> FrustumCullVoxelAllocator1;

uint NumVoxels;

Texture3D<float3> SpecularDirectionTexture0;
Texture3D<float3> SpecularDirectionTexture1;
uint DrawSpecularDirection;
float ProbeRadiusScale;

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void DebugTranslucencyLightingVolumeCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	FShaderPrintContext Context = InitShaderPrintContext(true, uint2(50, 50));

	if (all(DispatchThreadId == 0))
	{
		const float3 VolMin0 = View.TranslucencyLightingVolumeMin[0].xyz;
		const float3 VolMin1 = View.TranslucencyLightingVolumeMin[1].xyz;
		const float3 VolSize0 = 1.0f / View.TranslucencyLightingVolumeInvSize[0].xyz;
		const float3 VolSize1 = 1.0f / View.TranslucencyLightingVolumeInvSize[1].xyz;
		const float3 VolMax0 = VolMin0 + VolSize0;
		const float3 VolMax1 = VolMin1 + VolSize1;

		AddAABBTWS(Context, VolMin0, VolMax0, float4(0, 1, 0, 1));
		AddAABBTWS(Context, VolMin1, VolMax1, float4(1, 1, 0, 1));

		Print(Context, TEXT("Translucency Lighting Volume"), FontWhite);
		Newline(Context);

		Print(Context, TEXT("- Cascade 0:"), FontWhite);
		Newline(Context);

		Print(Context, TEXT("    - Size = "), FontWhite);
		Print(Context, VolSize0, FontYellow);
		Newline(Context);

		const uint VisibleVoxels0 = FrustumCullVoxelAllocator0[0];

		Print(Context, TEXT("    - # Visible Voxels = "), FontWhite);
		Print(Context, VisibleVoxels0, FontYellow);
		Print(Context, TEXT(" / "), FontYellow);
		Print(Context, NumVoxels, FontYellow);
		Print(Context, TEXT(" ("), FontYellow);
		Print(Context, VisibleVoxels0 * 100.0f / NumVoxels, FontYellow, 4, 1);
		Print(Context, TEXT("%)"), FontYellow);
		Newline(Context);

		const uint MarkedVoxels0 = VoxelAllocator0[0];

		Print(Context, TEXT("    - # Marked Voxels  = "), FontWhite);
		Print(Context, MarkedVoxels0, FontYellow);
		Print(Context, TEXT(" / "), FontYellow);
		Print(Context, NumVoxels, FontYellow);
		Print(Context, TEXT(" ("), FontYellow);
		Print(Context, MarkedVoxels0 * 100.0f / NumVoxels, FontYellow, 4, 1);
		Print(Context, TEXT("%)"), FontYellow);
		Newline(Context);

		Print(Context, TEXT("- Cascade 1:"), FontWhite);
		Newline(Context);

		Print(Context, TEXT("    - Size = "), FontWhite);
		Print(Context, VolSize1, FontYellow);
		Newline(Context);

		const uint VisibleVoxels1 = FrustumCullVoxelAllocator1[0];

		Print(Context, TEXT("    - # Visible Voxels = "), FontWhite);
		Print(Context, VisibleVoxels1, FontYellow);
		Print(Context, TEXT(" / "), FontYellow);
		Print(Context, NumVoxels, FontYellow);
		Print(Context, TEXT(" ("), FontYellow);
		Print(Context, VisibleVoxels1 * 100.0f / NumVoxels, FontYellow, 4, 1);
		Print(Context, TEXT("%)"), FontYellow);
		Newline(Context);

		const uint MarkedVoxels1 = VoxelAllocator1[0];

		Print(Context, TEXT("    - # Marked Voxels  = "), FontWhite);
		Print(Context, MarkedVoxels1, FontYellow);
		Print(Context, TEXT(" / "), FontYellow);
		Print(Context, NumVoxels, FontYellow);
		Print(Context, TEXT(" ("), FontYellow);
		Print(Context, MarkedVoxels1 * 100.0f / NumVoxels, FontYellow, 4, 1);
		Print(Context, TEXT("%)"), FontYellow);

		AddTextBackground(Context, float4(0, 0, 0, 0.25f));
	}

	// draw marked voxels

	if (VolumeMarkTexture0[DispatchThreadId] > 0)
	{
		const float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 0);
		const float3 VoxelExtent = View.TranslucencyLightingVolumeInvSize[0].w / 2;

		AddAABBTWS(Context, TranslatedWorldPosition - VoxelExtent, TranslatedWorldPosition + VoxelExtent, float4(0, 1, 0, 1));
	}

	if (VolumeMarkTexture1[DispatchThreadId] > 0)
	{
		const float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 1);
		const float3 VoxelExtent = View.TranslucencyLightingVolumeInvSize[1].w / 2;

		AddAABBTWS(Context, TranslatedWorldPosition - VoxelExtent, TranslatedWorldPosition + VoxelExtent, float4(1, 1, 0, 1));
	}

	// draw specular direction
	if (DrawSpecularDirection)
	{
		if (VolumeCascadeIndex == -1 || VolumeCascadeIndex == 0)
		{
			const float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 0);
			const float3 SpecularDirection = SpecularDirectionTexture0[DispatchThreadId] * 2.0f - 1.0f;

			AddLineTWS(Context, TranslatedWorldPosition, TranslatedWorldPosition + SpecularDirection * ProbeRadiusScale * 2, float4(1, 0, 0, 1));
		}

		if (VolumeCascadeIndex == -1 || VolumeCascadeIndex == 1)
		{
			const float3 TranslatedWorldPosition = GetTranslatedWorldPositionFromVolumeCoord(DispatchThreadId, 1);
			const float3 SpecularDirection = SpecularDirectionTexture1[DispatchThreadId] * 2.0f - 1.0f;

			AddLineTWS(Context, TranslatedWorldPosition, TranslatedWorldPosition + SpecularDirection * ProbeRadiusScale * 2, float4(1, 0, 0, 1));
		}
	}
}
#endif // DebugTranslucencyLightingVolumeCS
