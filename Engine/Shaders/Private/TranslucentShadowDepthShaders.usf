// Copyright Epic Games, Inc. All Rights Reserved.

#define SPHERICAL_OPACITY_FOR_SHADOW_DEPTHS 1

#define SCENE_TEXTURES_DISABLED 1

// Enable Substrate only when a material is specifying a Substrate material is specified.
#if !MATERIAL_IS_SUBSTRATE && SUBSTRATE_ENABLED
#undef SUBSTRATE_ENABLED
#define SUBSTRATE_ENABLED 0
#endif

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "ShadowDepthCommon.ush"

struct FTranslucencyShadowDepthVSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float ShadowDepth : TEXCOORD6;
	float4 PixelPosition : TEXCOORD7;
};

void SetShadowDepthOutputs(float4 WorldPosition, out float4 OutPosition, out float ShadowDepth)
{
	OutPosition = mul(WorldPosition, TranslucentDepthPass.ProjectionMatrix);

	// Clamp the vertex to the near plane if it is in front of the near plane
	// This has problems if some vertices of a triangle get clamped and others do not, also causes artifacts with non-ortho projections
	if (TranslucentDepthPass.bClampToNearPlane && OutPosition.z < 0)
	{
		OutPosition.z = 0.000001f;
		OutPosition.w = 1.0f;
	}

	#if SUPPORT_SHADOW_DEPTH_SPOT
		ShadowDepth = OutPosition.z;
	#else
		float InvMaxSubjectDepth = TranslucentDepthPass.InvMaxSubjectDepth;

		// Output linear, normalized depth
		ShadowDepth = OutPosition.z * InvMaxSubjectDepth;
		OutPosition.z = ShadowDepth * OutPosition.w;
	#endif
}

#if VERTEXSHADER
/** Shared vertex shader which transforms and outputs parameters necessary to evaluate opacity. */
void MainVS(
	FVertexFactoryInput Input,
	out FTranslucencyShadowDepthVSToPS OutParameters,
	out float4 OutPosition : SV_POSITION
	)
{
	ResolvedView = ResolveView();
	
	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 WorldPos = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPos.xyz, TangentToLocal);
	WorldPos.xyz += GetMaterialWorldPositionOffset(VertexParameters);
	ApplyMaterialFirstPersonTransform(VertexParameters, WorldPos.xyz);

	SetShadowDepthOutputs(
		WorldPos, 
		OutPosition, 
		OutParameters.ShadowDepth);

	OutParameters.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
	OutParameters.PixelPosition = WorldPos;
}
#endif // VERTEXSHADER

float TranslucentShadowStartOffset;

/** Pixel shader used to accumulate layer opacities in different channels based on the first translucent layer's depth. */
void MainOpacityPS( 
	FTranslucencyShadowDepthVSToPS Inputs,
	in float4 SvPosition : SV_Position,		// after all interpolators
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1
	)
{
	ResolvedView = ResolveView();

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Inputs.FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, 1);
	
	// Evaluate the mask for translucent materials
	GetMaterialClippingShadowDepth(MaterialParameters, PixelMaterialInputs);

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

	// Retrieve density from coverage and colored transmittance.
	float Density = 0.0f;

	float Sigma = 0.0f;
	float3 RefractionWorldNormal = MaterialParameters.CameraVector;
	if (SubstratePixelHeader.SubstrateTree.BSDFCount > 0)
	{
		const float3 V = MaterialParameters.CameraVector;

		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings(false /*bForceFullyRough*/, Substrate.bRoughDiffuse /*bRoughDiffuseEnabled*/, Substrate.PeelLayersAboveDepth, Substrate.bRoughnessTracking);
		SubstratePixelHeader.SubstrateUpdateTree(V, Settings);

		const float SubstrateTreeRootOperatorIndex = SubstrateData.OperatorIndex;
		const FSubstrateOperator RootOperator = SubstratePixelHeader.SubstrateTree.Operators[SubstrateTreeRootOperatorIndex];

		Density = (1.0 - RootOperator.Coverage) + RootOperator.Coverage * dot(RootOperator.ThroughputAlongV, (1.0f / 3.0f).xxx);
	}
#else
	float Density = GetMaterialOpacity(PixelMaterialInputs);
#endif

	Density *= GetMaterialTranslucentShadowDensityScale();

#if SUPPORT_SHADOW_DEPTH_SPOT
	/** Used to normalize the outputted depth */
	Inputs.ShadowDepth *= TranslucentDepthPass.InvMaxSubjectDepth;
#endif

	Inputs.ShadowDepth += TranslucentShadowStartOffset;

// Needs to match the corresponding define in ShadowProjectionCommon.usf
#define USE_FOURIER_OPACITY_MAP 1
#if USE_FOURIER_OPACITY_MAP

	// Fourier opacity shadow map
	float3 FrequencyScales0 = 2.0 * PI * float3(1, 2, 3);

	// Calculate the sin and cos wave scales for each frequency based on the current fragment's depth
	float3 CosCoefficients0;
	float3 SinCoefficients0;
	sincos(FrequencyScales0 * Inputs.ShadowDepth, SinCoefficients0, CosCoefficients0);

	float IntegratedDensity = -2 * log(max(1.0 - Density, .00001f));

	// X stores the cos coefficient at depth 0, which simplifies to just IntegratedDensity
	OutColor0 = float4(IntegratedDensity, IntegratedDensity * CosCoefficients0);
	OutColor1 = float4(0, IntegratedDensity * SinCoefficients0);

#else

	// Opacity shadow map
	float LayerSize = 1.0f / 15.0f;

	float4 LayerDepths0 = float4(0, 1, 2, 3) * LayerSize;
	float4 LayerDepths1 = float4(4, 5, 6, 7) * LayerSize;
	float4 LayerDepths2 = float4(8, 9, 10, 11) * LayerSize;
	float4 LayerDepths3 = float4(12, 13, 14, 15) * LayerSize;

	// Setup a linear falloff for density based on the distance of the current pixel to each layer
	// Layer density will be Density at the depth of the layer, and 0 at the depth of the next layer further from the light
	float4 LayerDensities0 = lerp(0, Density, saturate((LayerDepths0 + LayerSize - Inputs.ShadowDepth) / LayerSize));
	float4 LayerDensities1 = lerp(0, Density, saturate((LayerDepths1 + LayerSize - Inputs.ShadowDepth) / LayerSize));
	float4 LayerDensities2 = lerp(0, Density, saturate((LayerDepths2 + LayerSize - Inputs.ShadowDepth) / LayerSize));
	float4 LayerDensities3 = lerp(0, Density, saturate((LayerDepths3 + LayerSize - Inputs.ShadowDepth) / LayerSize));

	OutColor0 = LayerDensities0;
	OutColor1 = LayerDensities1;
	OutColor2 = LayerDensities2;
	OutColor3 = LayerDensities3;

#endif
}