// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=========================================================================
	TransmissionCommon.ush: Subsurface scattering transmission utilities.
=========================================================================*/

#include "SubsurfaceProfileCommon.ush"
#include "BurleyNormalizedSSSCommon.ush"

struct FTransmissionProfileParams
{
	float ExtinctionScale;
	float NormalScale;
	float ScatteringDistribution;
	float OneOverIOR;
};

FTransmissionProfileParams InitTransmissionProfileParams()
{
	FTransmissionProfileParams Out;
	Out.ExtinctionScale			= 1.0f;
	Out.NormalScale				= 0.08f;
	Out.ScatteringDistribution	= 0.0f;
	Out.OneOverIOR				= 1.0f;
	return Out;
}

FTransmissionProfileParams GetTransmissionProfileParams(uint SubsurfaceProfileInt)
{
	FTransmissionProfileParams Result;

	//X:ExtinctionScale, Y:Normal Scale, Z:ScatteringDistribution, W:OneOverIOR
	float4 Data = View.SSProfilesTexture.Load(int3(SSSS_TRANSMISSION_OFFSET, SubsurfaceProfileInt, 0));

	Result.ExtinctionScale = DecodeExtinctionScale(Data.x);
	Result.NormalScale = Data.y;
	Result.ScatteringDistribution = DecodeScatteringDistribution(Data.z);
	Result.OneOverIOR = Data.w;

	return Result;
}

float4 GetTransmissionProfile(uint SubsurfaceProfileInt, float Thickness)
{
	// 0..255, which SubSurface profile to pick
	const float2 TextureSize = View.SSProfilesTextureSizeAndInvSize.xy;
	float ProfileOffset = BSSS_TRANSMISSION_PROFILE_OFFSET;
	float2 uv = float2((ProfileOffset + Thickness / SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE * (BSSS_TRANSMISSION_PROFILE_SIZE - 1) + 0.5f), SubsurfaceProfileInt + 0.5f) / (TextureSize.xy);
	return Texture2DSampleLevel(View.SSProfilesTexture, View.SSProfilesTransmissionSampler, uv, 0);
}

float EncodeOpticalDepthToShadowMask(float OpticalDepth, float MaxOpticalDepth = SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE)
{
	return 1 - (OpticalDepth / MaxOpticalDepth);
}

float DecodeOpticalDepthFromShadowMask(float EncodedOpticalDepth)
{
	return (1 - EncodedOpticalDepth) * SSSS_MAX_TRANSMISSION_PROFILE_DISTANCE;
}

#include "FastMath.ush"

// Determine the distance that the light traveled through the subsurface object
// This assumes that anything between this subsurface pixel and the light was also a subsurface material,
// As a result, subsurface materials receive leaked light based on how transparent they are
#define GetSubSurfaceTransmissionGeneric(T) T GetSubSurfaceTransmission(T OccluderDistance, float SubsurfaceDensity) { return saturate(FastExp(-OccluderDistance * SubsurfaceDensity)); }
GetSubSurfaceTransmissionGeneric(float)
GetSubSurfaceTransmissionGeneric(float4)
