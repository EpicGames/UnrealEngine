// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

/*=============================================================================
	TransmissionThickness.ush: functions to calculate optical depth required to evaluate transmission in shadow shaders.
=============================================================================*/
#include "TransmissionCommon.ush"

float4x4 ScreenToShadowMatrix;

// x is start, z is end,From 1 to 32 sample
// .x:DepthBias, .y:SlopeDepthBias, .z:MaxSlopeDepthBias, .w: MaxSubjectZ - MinSubjectZ
float4 ProjectionDepthBiasParameters;

// Poisson disks generated with http://www.coderhaus.com/?p=11
// Combine all Poisson disk togeather, borrow idea from Jorge's multiple sample single scattering 
// http://www.iryoku.com/translucency/

int PoissonMethod;
static int2 PoissonStartAndEnd[] =
{
int2(0,  1),
int2(1,  3),
int2(3,  7),
int2(7, 15),
int2(15, 31),
int2(31, 63),
};

static const float2 PoissonDisk[] =
{
float2(0.402211,  0.126575),
float2(0.297056,  0.616830),
float2(0.298156, -0.001704),
float2(0.019369,  0.395482),
float2(-0.066918, -0.367739),
float2(-0.955010,  0.372377),
float2(0.800057,  0.120602),
float2(-0.7494944, 0.1827986),
float2(-0.8572887, -0.4169083),
float2(-0.1087135, -0.05238153),
float2(0.1045462, 0.9657645),
float2(-0.0135659, -0.698451),
float2(-0.4942278, 0.7898396),
float2(0.7970678, -0.4682421),
float2(0.8084122, 0.533884),
float2(0.4036454, -0.793536),
float2(0.3826454, -0.2730118),
float2(-0.04124885, -0.5971786),
float2(-0.3709261, -0.9179904),
float2(-0.3795351, -0.3353493),
float2(-0.3154466, 0.1069074),
float2(-0.7671808, -0.6143452),
float2(-0.4865215, 0.6395131),
float2(0.2359872, 0.1510548),
float2(0.03092861, 0.7309022),
float2(-0.82846, -0.1055831),
float2(-0.8732378, 0.3034171),
float2(0.9268684, -0.116035),
float2(0.6980102, 0.3764873),
float2(0.8239923, -0.515003),
float2(0.5084407, 0.7533528),
float2(-0.975402, -0.0711386),
float2(-0.920347, -0.41142),
float2(-0.883908, 0.217872),
float2(-0.884518, 0.568041),
float2(-0.811945, 0.90521),
float2(-0.792474, -0.779962),
float2(-0.614856, 0.386578),
float2(-0.580859, -0.208777),
float2(-0.53795, 0.716666),
float2(-0.515427, 0.0899991),
float2(-0.454634, -0.707938),
float2(-0.420942, 0.991272),
float2(-0.261147, 0.588488),
float2(-0.211219, 0.114841),
float2(-0.146336, -0.259194),
float2(-0.139439, -0.888668),
float2(0.0116886, 0.326395),
float2(0.0380566, 0.625477),
float2(0.0625935, -0.50853),
float2(0.125584, 0.0469069),
float2(0.169469, -0.997253),
float2(0.320597, 0.291055),
float2(0.359172, -0.633717),
float2(0.435713, -0.250832),
float2(0.507797, -0.916562),
float2(0.545763, 0.730216),
float2(0.56859, 0.11655),
float2(0.743156, -0.505173),
float2(0.736442, -0.189734),
float2(0.843562, 0.357036),
float2(0.865413, 0.763726),
float2(0.872005, -0.927)
};

float CalcShadowLinearDepth(float shadowDepth, float2 param)
{
	return 1.0f / (shadowDepth * param.x - param.y);
}

// OpticalDepth = depth * extinction. Extinction is oftened, wrongly, referred to as Density in UE legacy shading models.
float CalculateOpticalDepth(float ShadowmapDepth, float SceneDepth, float Extinction, float NormalScale, float NoL)
{
	// Determine the distance that the light traveled through the subsurface object
	// This assumes that anything between this subsurface pixel and the light was also a subsurface material,
	// As a result, subsurface materials receive leaked light based on how transparent they are
	float Thickness = (SceneDepth - ShadowmapDepth);	// unit = cm
	
	// Calculate optical depth from the distance travel inside a medium with a given Extinction.
	float OpticalDepth = Thickness * Extinction;		// unitless

	// Manipulate optical depth based on NoL and normal.
	OpticalDepth = clamp(abs(OpticalDepth > 0 ? OpticalDepth + NormalScale : max(0.0, OpticalDepth * NoL + NormalScale)), 0.15, 5) + 0.25;
	// Never shadow from depths that were never written to (max depth value)

	return OpticalDepth;
}

float CalculateEncodedOpticalDepth(float3 ViewPos, float3 LightPos, float3 WorldNormal, FTransmissionProfileParams TransmissionParams, FPCFSamplerSettings Settings)
{
	///Get -NoL, maybe unnecessary.
	float4 HomogeneousWorldPosition = mul(float4(ViewPos.xyz, 1), DFHackToFloat(PrimaryView.ScreenToWorld));
	float3 WorldPosition = HomogeneousWorldPosition.xyz / HomogeneousWorldPosition.w;
	float3 ToLight = LightPos.xyz - WorldPosition;
	float DistanceSqr = dot(ToLight, ToLight);
	float3 L = ToLight * rsqrt(DistanceSqr);
	float NoL = 1;// dot(-L, WorldNormal);

	//Shrink Pos in View Space.
	float3 ViewNormal = mul(WorldNormal, (float3x3)View.TranslatedWorldToView);

	//Shrink world position with neg worldNormal to get thickness
	//These scalar const is based on adjust expericence.
	float Extinction = TransmissionParams.ExtinctionScale;
	float NormalScale = TransmissionParams.NormalScale * 0.5;

	float ShadowFilterRadius = ShadowBufferSize.w;

	float4 ShrinkedPos = float4(ViewPos.xyz - (NormalScale)* ViewNormal.xyz, 1.0);
	float4 ShrinkedScreenPos = mul(ShrinkedPos, ScreenToShadowMatrix);


	ShrinkedScreenPos.xy = ShrinkedScreenPos.xy / ShrinkedScreenPos.w;
	float ExtinctionTimesDepthScale = Extinction * ProjectionDepthBiasParameters.w;

	Settings.SceneDepth = (1.0f - ShrinkedScreenPos.z) + ProjectionDepthBiasParameters.x;

	float OpticalDepthSum = 0.0;
	uint PoissonMethod = 4;
	float SampleNum = PoissonStartAndEnd[PoissonMethod].y - PoissonStartAndEnd[PoissonMethod].x;

	for (int i = PoissonStartAndEnd[PoissonMethod].x; i < PoissonStartAndEnd[PoissonMethod].y; i++)
	{
		float2 DeltUv = PoissonDisk[i] * ShadowFilterRadius;
		float2 ShadowUv = ShrinkedScreenPos.xy + DeltUv;
		float ShadowmapDepth = Texture2DSampleLevel(Settings.ShadowDepthTexture, Settings.ShadowDepthTextureSampler, ShadowUv.xy, 0).r;
		float OpticalDepth = CalculateOpticalDepth(ShadowmapDepth, Settings.SceneDepth, ExtinctionTimesDepthScale, NormalScale, NoL);
		OpticalDepthSum += (OpticalDepth);
	}
	float OpticalDepth = OpticalDepthSum / SampleNum;

	return EncodeOpticalDepthToShadowMask(OpticalDepth);

}



/** Computes thickness for a given world position from a cubemap shadowmap used on a point light. */
float CalculateEncodedOpticalDepth(float3 WorldNormal, FTransmissionProfileParams TransmissionParams, float3 WorldSampleToLightVec, float2 ProjParam, float LightInvRadius, float DepthBias)
{
	float OpticalDepth = 1;

	float DepthScale = TransmissionParams.ExtinctionScale;
	float NormalScale = TransmissionParams.NormalScale * 0.5;

	//Shrink pos aligh -WorldNormal
	float3 Normal = WorldNormal;
	float3 LightRelativeBiasedPosition = -WorldSampleToLightVec - Normal * NormalScale;
	float DistanceSqr = dot(WorldSampleToLightVec, WorldSampleToLightVec);
	float3 L = WorldSampleToLightVec * rsqrt(DistanceSqr);
	float NoL = pow(saturate(1 * dot(-L, Normal)), 1);
	float Distance = length(WorldSampleToLightVec);

	float ShadowFilterRadius = 1.0f;

	BRANCH // Skip pixels outside of the light's influence
	if (Distance * LightInvRadius < 1.0f)
	{
		float3 NormalizedLightVector = WorldSampleToLightVec / Distance, SideVector = 0, UpVector = 0;
		int CubeFaceIndex = 0;
		CalcCubemapVectorAndFaceIndex(NormalizedLightVector, ShadowFilterRadius, SideVector, UpVector, CubeFaceIndex);
		// Transform the position into shadow space
		float4 ShadowPosition = mul(float4(LightRelativeBiasedPosition, 1), ShadowViewProjectionMatrices[CubeFaceIndex]);

		// Calculate the Z buffer value that would have been stored for this position in the shadow map
		float CompareDistance = ShadowPosition.z / ShadowPosition.w;
		float ShadowDepthBias = -DepthBias / ShadowPosition.w;
		
		//Shadow Depth range Near to far, so no need mul far here.
		float Extinction = TransmissionParams.ExtinctionScale;
		float ExtinctionTimesDepthScale = Extinction * (10.0f / LightInvRadius);
		float SceneDepth = CalcShadowLinearDepth(CompareDistance + ShadowDepthBias, ProjParam) * LightInvRadius;

		OpticalDepth = 0;

		UNROLL for (int i = 0; i < 5; ++i)
		{
			float2 DeltUv = DiscSamples5[i] * ShadowFilterRadius;
			float3 SamplePos = NormalizedLightVector + SideVector * DeltUv.x + UpVector * DeltUv.y;
	#if USE_SEPARATE_SHADOW_DEPTH_CUBE_TEXTURE
			float ShadowSample = TextureCubeSampleLevel(ShadowDepthCubeTexture2, ShadowDepthTextureSampler, SamplePos, 0.0).r;
	#else
            float ShadowSample = TextureCubeSampleDepthLevel(ShadowDepthCubeTexture, ShadowDepthTextureSampler, SamplePos, 0.0);
	#endif
			float ShadowDepth = CalcShadowLinearDepth(ShadowSample, ProjParam) * LightInvRadius;
			OpticalDepth += CalculateOpticalDepth(ShadowDepth, SceneDepth, ExtinctionTimesDepthScale, NormalScale, NoL);
		}
		OpticalDepth /= 5;

	}
	return EncodeOpticalDepthToShadowMask(OpticalDepth);
}
