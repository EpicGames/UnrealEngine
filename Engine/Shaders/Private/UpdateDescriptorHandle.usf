// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UpdateDescriptorHandle.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

#if COMPILER_METAL_SHADER_CONVERTER
struct MetalDescriptor
{
    uint64_t gpuVA;
    uint64_t textureViewID;
    uint64_t metadata;
};
typedef MetalDescriptor Descriptor;
#else
struct DefaultDescriptor
{
    uint64_t gpuIndex;
};
typedef DefaultDescriptor Descriptor;
#endif

uint NumUpdates;

StructuredBuffer<uint> DescriptorIndices;
StructuredBuffer<Descriptor> DescriptorEntries;
RWStructuredBuffer<Descriptor> OutputData;

[numthreads(1, 1, 1)]
void MainCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	for(uint Idx = 0; Idx < NumUpdates; Idx++)
	{
        uint DestEntryIdx = DescriptorIndices[Idx];
        OutputData[DestEntryIdx] = DescriptorEntries[Idx];
	}
}
