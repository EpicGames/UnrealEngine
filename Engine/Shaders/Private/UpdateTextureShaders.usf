// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UpdateTextureShaders.usf: Compute shaders for copying and updating textures.
=============================================================================*/

#include "Common.ush"

Buffer<int> SrcBuffer;
RWTexture3D<int4> DestTexture3D;
uint SrcDepthPitch; //number of SrcBuffer entries between z slices of the source volume
uint4 DestPos; // xyz = starting offset in destination volume texture
uint4 DestSize; //syz = size of the volume to update
uint2 SrcPitch; //x = number of SrcBuffer elements in a row of the source area, y = Number of components per texel in the source.

[numthreads(8, 8, 1)]
void UpdateTexture3DSubresourceCS(uint3 ThreadId : SV_DispatchThreadID)
{
	if (all(ThreadId.xyz < DestSize.xyz))  
	{
		int3 TexturePixelOffset = ThreadId.xyz + (int3)DestPos.xyz;

		//supports textures with 1-4 components.  Textures with  < 4 components will simply ignore writes to the extra components via channel masking.
		int BufferOffset = (ThreadId.x * SrcPitch.y) + (ThreadId.y * SrcPitch.x) + (ThreadId.z * SrcDepthPitch);
		DestTexture3D[TexturePixelOffset] = int4(SrcBuffer[BufferOffset].x, SrcBuffer[BufferOffset + 1].x, SrcBuffer[BufferOffset + 2].x, SrcBuffer[BufferOffset + 3].x);		
	}
}

Buffer<uint4> SrcCopyBuffer;
RWBuffer<uint4> DestBuffer;

//.x Num elements per thread.
uint CopyElementsPerThread;

[numthreads(64,1,1)]
void CopyData2CS(uint3 ThreadGroupId : SV_GroupID, uint3 GroupThreadId : SV_GroupThreadID)
{
	int Offset1 = (ThreadGroupId.x * 2 + 0) * 64 + GroupThreadId.x;
	int Offset2 = (ThreadGroupId.x * 2 + 1) * 64 + GroupThreadId.x;
	DestBuffer[Offset1] = SrcCopyBuffer[Offset1];
	DestBuffer[Offset2] = SrcCopyBuffer[Offset2];	
}

[numthreads(64,1,1)]
void CopyData1CS(uint3 ThreadId : SV_DispatchThreadID)
{
	uint offset = ThreadId.x;	
	DestBuffer[offset + 0].xyzw = SrcCopyBuffer.Load(offset + 0).xyzw;	
}

#if defined(COMPONENT_TYPE)

Buffer<COMPONENT_TYPE> TSrcBuffer;
RWTexture2D<COMPONENT_TYPE> TDestTexture;
uint3 TSrcPosPitch;
uint4 TDestPosSize; // xy = pos, zw = Size of the destination sub rect

[numthreads(8,8,1)]
void TUpdateTexture2DSubresourceCS( uint2 ThreadId : SV_DispatchThreadID )
{
	if( all( ThreadId.xy < TDestPosSize.zw ) )
	{
		int2 DstTextureOffset = ThreadId.xy + TDestPosSize.xy;
		int2 SrcTextureOffset = ThreadId.xy + TSrcPosPitch.xy;
		int BufferOffset = (SrcTextureOffset.y * TSrcPosPitch.z) + SrcTextureOffset.x;
		TDestTexture[DstTextureOffset] = TSrcBuffer[BufferOffset];
	}
}

#endif // COMPONENT_TYPE
