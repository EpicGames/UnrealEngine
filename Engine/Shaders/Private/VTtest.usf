// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

void VirtualTextureTestPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0
	)
{
	OutColor = 0;
}