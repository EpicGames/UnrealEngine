// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRSShadingRateCommon.ush"
#include "../Common.ush"
#include "../ColorSpace.ush"

float4 ViewRect;
float EdgeThreshold;
float ConservativeEdgeThreshold;
float2 InvLuminanceTextureSize;

Texture2D<float> LuminanceTexture;
SamplerState LuminanceSampler;
RWTexture2D<uint> HardwareShadingRateImage;
RWTexture2D<uint> SoftwareShadingRateImage;

#define SOBEL_DIMENSION 3

#define HW_TILES_PER_GROUP_X ((THREADGROUP_SIZE_X * 2) / HARDWARE_TILE_SIZE)
#define HW_TILES_PER_GROUP_Y ((THREADGROUP_SIZE_Y * 2) / HARDWARE_TILE_SIZE)
#define HW_TILES_PER_GROUP (HW_TILES_PER_GROUP_X * HW_TILES_PER_GROUP_Y)
groupshared uint GroupHWUnsupported[HW_TILES_PER_GROUP];

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void CalculateShadingRateImage(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId    : SV_GroupThreadID,
	uint3 GroupId          : SV_GroupID,
	uint  GroupIndex       : SV_GroupIndex)
{
	const uint VRSBitMask = D3D12_SHADING_RATE_2X2 | (D3D12_SHADING_RATE_2X2 << CONSERVATIVE_SHADING_RATE_SHIFT);

	const uint2 QuadCoord = DispatchThreadId.xy;
	const uint2 ViewSize = uint2(ViewRect.zw - ViewRect.xy);

	// Threads outside the view rect or at the border don't enforce any rate restriction.
	uint UnsupportedShadingRate = 0u;

	const bool bFootprintInsideViewport = all(QuadCoord > 0u) && all((QuadCoord << 1u) + 2u < ViewSize);

	if (bFootprintInsideViewport)
	{
		const float2 Base = float2(int2(ViewRect.xy) + int2(QuadCoord << 1u)) * InvLuminanceTextureSize;
		const float4 GTL = LuminanceTexture.Gather(LuminanceSampler, Base);
		const float4 GTR = LuminanceTexture.Gather(LuminanceSampler, Base, int2(2, 0));
		const float4 GBL = LuminanceTexture.Gather(LuminanceSampler, Base, int2(0, 2));
		const float4 GBR = LuminanceTexture.Gather(LuminanceSampler, Base, int2(2, 2));

		float L[4][4];
		L[0][0] = GTL.w; L[1][0] = GTL.z; L[2][0] = GTR.w; L[3][0] = GTR.z;
		L[0][1] = GTL.x; L[1][1] = GTL.y; L[2][1] = GTR.x; L[3][1] = GTR.y;
		L[0][2] = GBL.w; L[1][2] = GBL.z; L[2][2] = GBR.w; L[3][2] = GBR.z;
		L[0][3] = GBL.x; L[1][3] = GBL.y; L[2][3] = GBR.x; L[3][3] = GBR.y;

		const float SobelX[SOBEL_DIMENSION][SOBEL_DIMENSION] =
		{
			{-1.0, 0.0, 1.0},
			{-2.0, 0.0, 2.0},
			{-1.0, 0.0, 1.0}
		};

		const float SobelY[SOBEL_DIMENSION][SOBEL_DIMENSION] =
		{
			{-1.0, -2.0, -1.0},
			{ 0.0,  0.0,  0.0},
			{ 1.0,  2.0,  1.0}
		};

		UNROLL
		for (int j = 0; j < 2; j++)
		{
			UNROLL
			for (int i = 0; i < 2; i++)
			{
				float SobelXSum = 0.0;
				float SobelYSum = 0.0;

				UNROLL
				for (int kx = 0; kx < SOBEL_DIMENSION; kx++)
				{
					UNROLL
					for (int ky = 0; ky < SOBEL_DIMENSION; ky++)
					{
						const float LumaValue = L[i + kx][j + ky];
						SobelXSum += SobelX[ky][kx] * LumaValue;
						SobelYSum += SobelY[ky][kx] * LumaValue;
					}
				}

				const float AbsX = abs(SobelXSum);
				const float AbsY = abs(SobelYSum);
				if (AbsX > EdgeThreshold)             UnsupportedShadingRate |= D3D12_SHADING_RATE_2X1;
				if (AbsX > ConservativeEdgeThreshold) UnsupportedShadingRate |= (D3D12_SHADING_RATE_2X1 << CONSERVATIVE_SHADING_RATE_SHIFT);
				if (AbsY > EdgeThreshold)             UnsupportedShadingRate |= D3D12_SHADING_RATE_1X2;
				if (AbsY > ConservativeEdgeThreshold) UnsupportedShadingRate |= (D3D12_SHADING_RATE_1X2 << CONSERVATIVE_SHADING_RATE_SHIFT);
			}
		}
	}

	const bool bQuadInView = !any((QuadCoord << 1u) >= ViewSize);

#if OUTPUT_SOFTWARE_IMAGE
	if (bQuadInView)
	{
		SoftwareShadingRateImage[QuadCoord] = VRSBitMask & ~UnsupportedShadingRate;
	}
#endif

#if OUTPUT_HARDWARE_IMAGE
	const uint2 HwTileLocal     = (GroupThreadId.xy * 2u) / HARDWARE_TILE_SIZE;
	const uint  HwTileIndex       = HwTileLocal.y * HW_TILES_PER_GROUP_X + HwTileLocal.x;
	const bool  bIsHwTileWriter = all((GroupThreadId.xy * 2u) % HARDWARE_TILE_SIZE == 0u);

	if (bIsHwTileWriter)
	{
		GroupHWUnsupported[HwTileIndex] = 0u;
	}
	GroupMemoryBarrierWithGroupSync();

	if (UnsupportedShadingRate != 0u)
	{
		InterlockedOr(GroupHWUnsupported[HwTileIndex], UnsupportedShadingRate);
	}
	GroupMemoryBarrierWithGroupSync();

	if (bIsHwTileWriter && bQuadInView)
	{
		const uint2 HwTileOut = (QuadCoord << 1u) / HARDWARE_TILE_SIZE;
		HardwareShadingRateImage[HwTileOut] = VRSBitMask & ~GroupHWUnsupported[HwTileIndex];
	}
#endif
}
