// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRSShadingRateCommon.ush"
#include "../Common.ush"

#if FRAGMENT_DENSITY_MAP
	RWTexture2D<float2> RWOutputTexture;
	Texture2D<float2> SourceTexture0;
	Texture2D<float2> SourceTexture1;
#else
	RWTexture2D<uint> RWOutputTexture;
	Texture2D<uint> SourceTexture0;
	Texture2D<uint> SourceTexture1;
#endif

uint ShadingRateMax(uint InputRate0, uint InputRate1)
{
	uint2 SeparatedShadingRate0 = SeparateShadingRateAxes(InputRate0);
	uint2 SeparatedShadingRate1 = SeparateShadingRateAxes(InputRate1);
	uint2 SeparatedMaxRate = max(SeparatedShadingRate0, SeparatedShadingRate1);
	return CombineShadingRateAxes(SeparatedMaxRate);
}

[numthreads(THREADGROUP_SIZE_XY, THREADGROUP_SIZE_XY, 1)]
void CombineShadingRateTextures(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 TexelCoord = DispatchThreadId.xy;

#if FRAGMENT_DENSITY_MAP
	// "Max" refers to the roughest shadng rate, which corresponds to the lowest fragment density
	RWOutputTexture[TexelCoord] = min(SourceTexture0[TexelCoord], SourceTexture1[TexelCoord]);
#else
	RWOutputTexture[TexelCoord] = ShadingRateMax(SourceTexture0[TexelCoord], SourceTexture1[TexelCoord]);
#endif
}

