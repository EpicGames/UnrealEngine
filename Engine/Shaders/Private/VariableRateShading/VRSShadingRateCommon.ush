// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Palette shading definitions from d3d12.h
#define D3D12_AXIS_SHADING_RATE_1X	0x0
#define D3D12_AXIS_SHADING_RATE_2X	0x1
#define D3D12_AXIS_SHADING_RATE_4X	0x2

#define D3D12_SHADING_RATE_1X1 0x0
#define D3D12_SHADING_RATE_1X2 0x1
#define D3D12_SHADING_RATE_2X1 0x4
#define D3D12_SHADING_RATE_2X2 0x5
#define D3D12_SHADING_RATE_2X4 0x6
#define D3D12_SHADING_RATE_4X2 0x9
#define D3D12_SHADING_RATE_4X4 0xa

#define D3D12_SHADING_RATE_X_AXIS_SHIFT 2
#define D3D12_SHADING_RATE_VALID_MASK 3
#define D3D12_SHADING_RATE_MAX_BITS 4

// Used by Contrast Adaptive Shading to encode both full and conservative shading rates in a single uint
#define CONSERVATIVE_SHADING_RATE_SHIFT D3D12_SHADING_RATE_MAX_BITS

// Palette shading rate preview colors
#define RATE_1X1_COLOR float3(1.0, 0.0, 0.0) // Red
#define RATE_1X2_COLOR float3(1.0, 0.5, 0.0) // Orange
#define RATE_2X1_COLOR float3(1.0, 1.0, 0.0) // Yellow
#define RATE_2X2_COLOR float3(0.0, 1.0, 0.0) // Green
#define RATE_2X4_COLOR float3(0.0, 0.0, 1.0) // Blue
#define RATE_4X2_COLOR float3(1.0, 0.0, 1.0) // Purple
#define RATE_4X4_COLOR float3(1.0, 0.5, 1.0) // Pink

// Utilities to go to/from a single uint representing the shading rate of both axes <--> a uint2 representing each per-axis rate
uint2 SeparateShadingRateAxes(uint CombinedShadingRate)
{
	uint2 SeparatedShadingRate;
	SeparatedShadingRate.x = D3D12_SHADING_RATE_VALID_MASK & (CombinedShadingRate >> D3D12_SHADING_RATE_X_AXIS_SHIFT);
	SeparatedShadingRate.y = D3D12_SHADING_RATE_VALID_MASK & CombinedShadingRate;
	return SeparatedShadingRate;
}

uint CombineShadingRateAxes(uint2 SeparatedShadingRate)
{
	return ((SeparatedShadingRate.x << D3D12_SHADING_RATE_X_AXIS_SHIFT) | SeparatedShadingRate.y);
}


// Utilities to convert to/from a palette shading rate <--> fragment density
// These conversions are approximate and may exactly match how the requested rate/density is interpreted by the driver
float AxisShadingRateToFragmentDensity(uint AxisShadingRate)
{
	float AxisFragmentDensity = 1.0f;

	if (AxisShadingRate == D3D12_AXIS_SHADING_RATE_2X)
	{
		AxisFragmentDensity = 0.5f;
	}
	else if (AxisShadingRate == D3D12_AXIS_SHADING_RATE_4X)
	{
		AxisFragmentDensity = 0.25f;
	}

	return AxisFragmentDensity;
}

uint AxisFragmentDensityToShadingRate(float AxisFragmentDensity)
{
	const float DensityDelta = 0.004f; // Account for imprecision when densities are 8-bit

	uint AxisShadingRate = D3D12_AXIS_SHADING_RATE_1X;

	if (AxisFragmentDensity <= 0.25f + DensityDelta)
	{
		AxisShadingRate = D3D12_AXIS_SHADING_RATE_4X;
	}
	else if (AxisFragmentDensity <= 0.5f + DensityDelta)
	{
		AxisShadingRate = D3D12_AXIS_SHADING_RATE_2X;
	}

	return AxisShadingRate;
}

float2 ShadingRateToFragmentDensity(uint ShadingRate)
{
	uint2 SeparatedShadingRate = SeparateShadingRateAxes(ShadingRate);
	float FragmentDensityX = AxisShadingRateToFragmentDensity(SeparatedShadingRate.x);
	float FragmentDensityY = AxisShadingRateToFragmentDensity(SeparatedShadingRate.y);
	return float2(FragmentDensityX, FragmentDensityY);
}

uint FragmentDensityToShadingRate(float2 FragmentDensity)
{
	uint2 SeparatedShadingRate = 0;
	SeparatedShadingRate.x = AxisFragmentDensityToShadingRate(FragmentDensity.x);
	SeparatedShadingRate.y = AxisFragmentDensityToShadingRate(FragmentDensity.y);

	// Filter invalid rates 1x4 and 4x1 --> convert to 2x4 and 4x2, respectively
	if (SeparatedShadingRate.x == D3D12_AXIS_SHADING_RATE_1X && SeparatedShadingRate.y == D3D12_AXIS_SHADING_RATE_4X)
	{
		SeparatedShadingRate.x = D3D12_AXIS_SHADING_RATE_2X;
	}
	if (SeparatedShadingRate.x == D3D12_AXIS_SHADING_RATE_4X && SeparatedShadingRate.y == D3D12_AXIS_SHADING_RATE_1X)
	{
		SeparatedShadingRate.y = D3D12_AXIS_SHADING_RATE_2X;
	}

	return CombineShadingRateAxes(SeparatedShadingRate);
}


// Utilities to convert a shading rate or fragment density into a preview overlay color
float4 VisualizeShadingRate(uint ShadingRate)
{
	float3 RateColor = 0;

	switch (ShadingRate)
	{
		case D3D12_SHADING_RATE_1X1:
			RateColor = RATE_1X1_COLOR;
			break;
		case D3D12_SHADING_RATE_1X2:
			RateColor = RATE_1X2_COLOR;
			break;
		case D3D12_SHADING_RATE_2X1:
			RateColor = RATE_2X1_COLOR;
			break;
		case D3D12_SHADING_RATE_2X2:
			RateColor = RATE_2X2_COLOR;
			break;
		case D3D12_SHADING_RATE_2X4:
			RateColor = RATE_2X4_COLOR;
			break;
		case D3D12_SHADING_RATE_4X2:
			RateColor = RATE_4X2_COLOR;
			break;
		case D3D12_SHADING_RATE_4X4:
			RateColor = RATE_4X4_COLOR;
			break;
	}

	return float4(RateColor.rgb, 0.5f);
}

float4 VisualizeFragmentDensity(float2 FragmentDensity)
{
	uint ApproximateEquivalentShadingRate = FragmentDensityToShadingRate(FragmentDensity);
	return VisualizeShadingRate(ApproximateEquivalentShadingRate);
}