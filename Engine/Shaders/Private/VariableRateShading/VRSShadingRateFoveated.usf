// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FixedFoveationVariableRateShading.usf
=============================================================================*/

#include "../Common.ush"

// Supplied defines:
// SHADING_RATE_1x1			- Shading rate of 1x1
// SHADING_RATE_1x2			- Shading rate of 1x2
// SHADING_RATE_2x1			- Shading rate of 2x1
// SHADING_RATE_2x2			- Shading rate of 2x2
// SHADING_RATE_2x4			- Shading rate of 2x4
// SHADING_RATE_4x2			- Shading rate of 4x2
// SHADING_RATE_4x4			- Shading rate of 4x4

// MAX_COMBINED_SOURCES_IN	- Maximum 	
// SHADING_RATE_TILE_WIDTH	- Shading rate tile width
// SHADING_RATE_TILE_HEIGHT - Shading rate tile height
// THREADGROUP_SIZEX		- Threadgroup size (X)
// THREADGROUP_SIZEY		- Threadgroup size (Y)

// Generated output shading rate attachment.
RWTexture2D<uint> RWOutputTexture;

// Horizontal and vertical field of view for the current view.
float2	FieldOfView;

// Center pixel coordinate for the left eye.
float2	LeftEyeCenterPixelXY;

// Center pixel coordinate for the right eye.
float2	RightEyeCenterPixelXY;

// Diagonal of the view in pixels, squared.
float	ViewDiagonalSquaredInPixels;
// Cutoff percentage for full-rate shading, foveation (0.0 to 1.0), squared. Outside this range will be shaded at half-rate in x and y.
float	FoveationFullRateCutoffSquared;
// Cutoff percentage for half-rate shading, foveation (0.0 to 1.0), squared. Outside this range will be shaded at quarter-rate in x and y.
float	FoveationHalfRateCutoffSquared;

// Bitfield indicating optional supporting things (defined in the EVRSGenerationFlags enum:)
// 	EVRSGenerationFlags::None = 0x0,
//	EVRSGenerationFlags::StereoRendering = 0x1,
//	EVRSGenerationFlags::SideBySideStereo = 0x2,
uint ShadingRateAttachmentGenerationFlags;

////////////////////////////////////////////////////////////////////////////////////////////////////
// Get the fractional offset of the pixel coordinate from the center of the eye-view target 
// 0.0 = center of view.
// 1.0 = diagonal far edge of view.
////////////////////////////////////////////////////////////////////////////////////////////////////

float GetFractionalOffsetFromEyeOrigin(float2 PixelPosition)
{
	if ((ShadingRateAttachmentGenerationFlags & SIDE_BY_SIDE_STEREO) != 0)
	{
		const float2 VecToLeft = PixelPosition - LeftEyeCenterPixelXY;
		const float2 VecToRight = PixelPosition - RightEyeCenterPixelXY;
		return min(dot(VecToLeft, VecToLeft), dot(VecToRight, VecToRight)) / ViewDiagonalSquaredInPixels;
	}
	else
	{
		// Single eye, just return this case.
		const float2 Vec = PixelPosition - LeftEyeCenterPixelXY;
		return dot(Vec, Vec) / ViewDiagonalSquaredInPixels;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// Get the shading rate at the specified pixel position, for HMD-based fixed or gaze-tracked foveation.
////////////////////////////////////////////////////////////////////////////////////////////////////

uint GetFoveatedShadingRate(uint2 PixelPositionIn)
{
	const float2 PixelPosition = float2((float)PixelPositionIn.x, (float)PixelPositionIn.y);
	const float FractionalOffset = GetFractionalOffsetFromEyeOrigin(PixelPosition);
	
	if (FractionalOffset > FoveationHalfRateCutoffSquared)
	{
		return SHADING_RATE_4x4;
	}

	if (FractionalOffset > FoveationFullRateCutoffSquared)
	{
		return SHADING_RATE_2x2;
	}

	return SHADING_RATE_1x1;
}


[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void GenerateShadingRateTexture(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	const uint2 TexelCoord = DispatchThreadId.xy;
	RWOutputTexture[TexelCoord] = GetFoveatedShadingRate(TexelCoord);
}
