// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRSShadingRateCommon.ush"
#include "../Common.ush"
#include "../ColorSpace.ush"

float4 ViewRect;
float DynamicResolutionScale;

RWTexture2D<float4> SceneColorOut;
#if FRAGMENT_DENSITY_MAP
	Texture2D<float2> VariableRateShadingTextureIn;
#else
	Texture2D<uint> VariableRateShadingTextureIn;
#endif

void PreviewVariableRateShadingTexturePS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	uint2 VRSDimension;
	VariableRateShadingTextureIn.GetDimensions(VRSDimension.x, VRSDimension.y);

	int2 VRSIndex = UVAndScreenPos.xy * VRSDimension;

#if FRAGMENT_DENSITY_MAP
	float2 FragmentDensity = VariableRateShadingTextureIn.Load(int3(VRSIndex,0)).rg;
	OutColor = VisualizeFragmentDensity(FragmentDensity);
#else
	uint ShadingRate = VariableRateShadingTextureIn.Load(int3(VRSIndex,0)).r & (15U);
	OutColor = VisualizeShadingRate(ShadingRate);
#endif
}
