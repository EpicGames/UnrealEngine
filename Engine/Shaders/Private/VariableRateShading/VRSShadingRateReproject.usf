// Copyright Epic Games, Inc. All Rights Reserved.

#include "VRSShadingRateCommon.ush"
#include "../Common.ush"
#include "../SceneTexturesCommon.ush"

float2 InputSRIDimensions;
float2 ScaledSRIDimensions;
float2 TextureDimensions;
float2 InvTextureDimensions;

float2 ScaledUVOffset;
float  InvDynamicResolutionScale;

Texture2D<uint> InputSRITexture;
RWTexture2D<uint> ScaledSRITexture;
RWTexture2D<uint> ScaledConservativeSRITexture;

groupshared float CachedDepth[THREADGROUP_SIZE / 2][THREADGROUP_SIZE / 2];

// This is a large approximation of reprojection in a couple of ways in favor of performance:
// 1. We only look at reprojection based on the camera, not velocity vectors. I.E. this will be incorrect for dynamic objects. In most cases
//    motion blur will hide this entirely, though I found no perceptable artifacts even with motion blur off
// 2. The reprojection only does one depth texture fetch in the center of each 16x16 pixel block.
//
// Both of these approximations are fine because VRS doesn't require a whole lot of accuracy since the side-effect is 
// just lower resolution in areas that are "wrong" rather than the type of ghosting you'd get from bad reprojection in something 
// like TAA. In practice, I found reprojection is only required in a really small set of scenarios since VRS already preserves 
// silhouettes anyways. The main artifact I saw was materials that have a strip of metal that's not on the edge of a mesh and can
// cause noticeble aliasing when it's got a bright specular highlight.
//
// Note that some of the above considerations do not apply for Software VRS - increased granularity leaves less allowance
// for incorrect reprojection, which can lead to flickering along edges where rough rates have been applied incorrectly.
float2 ReprojectBufferUV(uint3 DispatchThreadId, uint3 GroupThreadId, float2 BufferUV, bool bValidThread)
{
	// Only read depth once in the top left thread for each 2x2 block of thread IDs
	// This shader is primarily bound on reading from the depth buffer on Scarlett. At 4k, 
	// doing a single depth buffer tap that gets shared by each 2x2 block of threads ends 
	// up making the shader almost 4x faster
	uint2 CachedDepthIndex = uint2(GroupThreadId.x / 2, GroupThreadId.y / 2);
	if( GroupThreadId.x % 2 == 0 && GroupThreadId.y % 2 == 0 && bValidThread)
	{
		const float2 pixelUVSize = InvTextureDimensions;

		// Center the UV in the middle of the 2x2 block
		float2 UVOffset = ScaledUVOffset;
		float2 DepthUV = UVOffset + (float2(DispatchThreadId.xy) + float2(1, 1)) * pixelUVSize;
		DepthUV = clamp(DepthUV, View.BufferBilinearUVMinMax.xy, View.BufferBilinearUVMinMax.zw);
		CachedDepth[CachedDepthIndex.x][CachedDepthIndex.y] = LookupDeviceZ(DepthUV);
    }
	GroupMemoryBarrierWithGroupSync();

	if(bValidThread)
	{
		float DeviceZ = CachedDepth[CachedDepthIndex.x][CachedDepthIndex.y];  
		float2 ScreenPosition = ViewportUVToScreenPos(BufferUVToViewportUV(BufferUV));
		float4 ThisClip = float4(ScreenPosition, DeviceZ, 1);
		float4 PrevClip = mul(ThisClip, View.ClipToPrevClip);
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
		BufferUV = ViewportUVToBufferUV(ScreenPosToViewportUV(PrevScreen));
	}
	return BufferUV;
}

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void RescaleVariableRateShading(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	StereoSetupCS(0); // Needed to support ViewportUVToBufferUV / BufferUVToViewportUV, which read from ResolvedView under INSTANCED_STEREO || MOBILE_MULTI_VIEW

	bool bValidThread = DispatchThreadId.x < ScaledSRIDimensions.x && DispatchThreadId.y < ScaledSRIDimensions.y;
	
	uint shadingRate = D3D12_SHADING_RATE_2X2 | (D3D12_SHADING_RATE_2X2 << CONSERVATIVE_SHADING_RATE_SHIFT);
	float2 pixelUVSize = InvTextureDimensions;
	const float epsilon = 0.0001;

	const float2 HalfPixelOffset = float2(0.5, 0.5);

	float2 CenterUV = (float2(DispatchThreadId.xy) + HalfPixelOffset) * pixelUVSize + ScaledUVOffset;
	CenterUV = ReprojectBufferUV(DispatchThreadId, GroupThreadId, CenterUV, bValidThread); 

	// FXC doesn't allow a thread to terminate before GroupMemoryBarrierWithGroupSync so 
	// this return isn't allowed until right after ReprojectBufferUV
	if(!bValidThread)
	{
		return;
	}

	// When using splitscreen or stereo, the output reprojected SRI is double-wide, but the input SRIs are single-wide
	// So we need to adjust the coords used to sample from the input SRIs to always sample from a single-wide texture
	CenterUV = CenterUV - ScaledUVOffset;

	// Map from pre-scaled UV to post-scaled UV.
	float2 uvStart = CenterUV - pixelUVSize * 0.5 + epsilon;
	float2 uvEnd = CenterUV + pixelUVSize * 0.5 - epsilon;

	// Convert from UV space to Texel space of the source (pre-scaled) image
	int2 inputCoordStart = uvStart * InvDynamicResolutionScale * TextureDimensions.xy;
	int2 inputCoordEnd = uvEnd * InvDynamicResolutionScale * TextureDimensions.xy;
	
	// Clamp start to top left corner of the input SRI.
	inputCoordStart.x = max(inputCoordStart.x, 0.0f);
	inputCoordStart.y = max(inputCoordStart.y, 0.0f);

	// Clamp end to bottom right corner of the input SRI.
	inputCoordEnd.x = min(inputCoordEnd.x, InputSRIDimensions.x);
	inputCoordEnd.y = min(inputCoordEnd.y, InputSRIDimensions.y);

	// start.x or y will be greater than end.x or y iff both corners were out of view on the same side.
	// start is never clamped against the bottom-right and end is never clamped against the top-left
	// So the only way start can be greater than end, is if both corners were out of view and only
	// one of them got clamped passed the other.
	bool bOnScreen = all(inputCoordStart <= inputCoordEnd);

	if(bOnScreen)
	{
		for(uint x = inputCoordStart.x; x <= inputCoordEnd.x; x++)
		{
			for(uint y = inputCoordStart.y; y <= inputCoordEnd.y; y++)
			{
				shadingRate &= InputSRITexture[uint2(x, y)];
			}
		}
	}
	else
	{
		// Don't apply VRS to anything coming offscreen
		shadingRate = D3D12_SHADING_RATE_1X1 | (D3D12_SHADING_RATE_1X1 << CONSERVATIVE_SHADING_RATE_SHIFT);
	}

	ScaledSRITexture[DispatchThreadId.xy + ScaledUVOffset * TextureDimensions] = shadingRate & D3D12_SHADING_RATE_2X2;
	ScaledConservativeSRITexture[DispatchThreadId.xy + ScaledUVOffset * TextureDimensions] = (shadingRate >> CONSERVATIVE_SHADING_RATE_SHIFT) & D3D12_SHADING_RATE_2X2;
}
