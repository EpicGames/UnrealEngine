// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	GlobalVectorFieldShaders.usf: Shaders for building global vector fields.
==============================================================================*/

/*------------------------------------------------------------------------------
	Composite an animated vector field.
------------------------------------------------------------------------------*/

#if COMPOSITE_ANIMATED

#include "Common.ush"

Texture2D AtlasTexture;
SamplerState AtlasTextureSampler;
Texture3D NoiseVolumeTexture;
SamplerState NoiseVolumeTextureSampler;
RWTexture3D<float4> OutVolumeTexture;

void Extrude_X(float3 VolumeUV, out float2 ImageUV, out float3 AxisX, out float3 AxisY, out float3 AxisZ)
{
	ImageUV.x = VolumeUV.y;
	ImageUV.y = 1.0f - VolumeUV.z;
	AxisX = float3(0,1,0);
	AxisY = float3(0,0,1);
	AxisZ = float3(1,0,0);
}

void Extrude_Y(float3 VolumeUV, out float2 ImageUV, out float3 AxisX, out float3 AxisY)
{
	ImageUV.x = VolumeUV.x;
	ImageUV.y = 1.0f - VolumeUV.z;
	AxisX = float3(1,0,0);
	AxisY = float3(0,0,1);
}

void Extrude_Z(float3 VolumeUV, out float2 ImageUV, out float3 AxisX, out float3 AxisY)
{
	ImageUV.x = VolumeUV.x;
	ImageUV.y = 1.0f - VolumeUV.y;
	AxisX = float3(1,0,0);
	AxisY = float3(0,1,0);
}

void Revolve_HalfZ(float3 VolumeUV, out float2 ImageUV, out float3 AxisX, out float3 AxisY, out float3 AxisZ)
{
	const float2 PlaneDir = VolumeUV.xy * 2 - 1;
	//ImageUV.x = 0.5f * saturate(length(PlaneDir)) + 0.5f;
	ImageUV.x = saturate(length(PlaneDir));
	ImageUV.y = 1.0f - VolumeUV.z;
	AxisX = float3(normalize(PlaneDir),0);
	AxisY = float3(0,0,1);
	AxisZ = cross(AxisX,AxisY);
}

[numthreads(THREADS_X,THREADS_Y,THREADS_Z)]
void CompositeAnimatedVectorField(
	uint3 DispatchThreadId	: SV_DispatchThreadID,
	uint3 GroupId			: SV_GroupID,
	uint3 GroupThreadId		: SV_GroupThreadID,
	uint ThreadId			: SV_GroupIndex )
{
	float3 AxisX = float3(1,0,0);
	float3 AxisY = float3(0,1,0);
	float3 AxisZ = float3(0,0,1);
	float2 ImageUV = float2(0,0);

	// Compute the UV coordinate for this voxel within the volume.
	const float3 VolumeUV = DispatchThreadId * CVF.VoxelSize.xyz + CVF.VoxelSize.xyz * 0.5f;

	// Sample the noise texture.
	const float3 NoiseSample = Texture3DSampleLevel(NoiseVolumeTexture, NoiseVolumeTextureSampler, VolumeUV, 0).xyz;
	const float NoiseMagnitude = length(NoiseSample);
	const float3 NormalizedNoise = NoiseSample / (NoiseMagnitude + 0.001f);
	const float3 Noise = NormalizedNoise * min(NoiseMagnitude * CVF.NoiseScale, CVF.NoiseMax);

	// Determine the 2-D projection for this voxel based on the desired operation.
	if (CVF.Op == 0) // VFCO_Extrude
	{
		Extrude_X(VolumeUV, ImageUV, AxisX, AxisY, AxisZ);
	}
	else // VFCO_Revolve
	{
		Revolve_HalfZ(VolumeUV, ImageUV, AxisX, AxisY, AxisZ);
	}

	// Sample the two subimages and interpolate.
	float2 FrameA_UV = ImageUV.xy * CVF.FrameA.xy + CVF.FrameA.zw;
	float2 FrameB_UV = ImageUV.xy * CVF.FrameB.xy + CVF.FrameB.zw;
	float4 FrameA_Sample = Texture2DSampleLevel(AtlasTexture, AtlasTextureSampler, FrameA_UV, 0);
	float4 FrameB_Sample = Texture2DSampleLevel(AtlasTexture, AtlasTextureSampler, FrameB_UV, 0);
	float4 LerpedValue = lerp(FrameA_Sample, FrameB_Sample, CVF.FrameLerp);

	// Determine the direction and project back to the volume.
	float3 Direction = (LerpedValue.xyz * 2 - 1) * LerpedValue.w;
	float3 ProjectedDirection = Direction.x * AxisX + Direction.y * AxisY + Direction.z * AxisZ;

	// Write out this voxel's result.
	OutVolumeTexture[DispatchThreadId] = float4(Noise + ProjectedDirection, 0.0f);
}

#endif // #if COMPOSITE_ANIMATED
