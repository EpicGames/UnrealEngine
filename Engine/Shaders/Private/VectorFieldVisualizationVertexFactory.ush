// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VectorFieldVisualizationVertexFactory.usf: Vertex factory for GPU simulated particles.
=============================================================================*/

#define PARTICLE_SPRITE_FACTORY 1

#include "VertexFactoryCommon.ush"

/** Texture containing the vector field. */
Texture3D VectorFieldTexture;
SamplerState VectorFieldTextureSampler;

#undef GetInstanceIdFromVF
#define GetInstanceIdFromVF(VFInput)				(GetInstanceId(VFInput.InstanceId))

/**
 * Vertex attributes to fetch.
 */
struct FVertexFactoryInput
{
	/** Unique vertex ID. */
	uint VertexId	: SV_VertexID;
	/** Unique instance ID. */
	uint InstanceId : SV_InstanceID;
	
	VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() 
};

/**
 * Attributes to interpolate from the vertex shader to the pixel shader.
 */
struct FVertexFactoryInterpolantsVSToPS
{
	/** Dummy value to interpolate. */
	float2 DummyTexCoord : TEXCOORD0;
};

/**
 * Intermediate values computed in the vertex shader.
 */
struct FVertexFactoryIntermediates
{
	/** Dummy value. */
	float4 WorldPosition;
	/** Position before transformation to worldspace */
	float4 PositionInstanceSpace;
	/** Cached primitive and instance data */
	FSceneDataIntermediates SceneData;
};

FPrimitiveSceneData GetPrimitiveData(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData.Primitive;
}

/**
 * Computes intermediates for the given vertex.
 * @param Input - Vertex attributes.
 * @returns the computed intermediate values.
 */
FVertexFactoryIntermediates GetVertexFactoryIntermediates( FVertexFactoryInput Input )
{
	// Determine how to index in to the volume texture.
	const float3 VoxelSize = VectorFieldVis.VoxelSize.xyz;
	const float3 HalfVoxelOffset = VoxelSize.xyz * 0.5f;
	const float3 VoxelMultipliers = float3(1, VoxelSize.x, VoxelSize.x * VoxelSize.y);
	const float3 VoxelIndex = floor((float3)GetInstanceId(Input.InstanceId).xxx * VoxelMultipliers.xyz);
	const float3 VoxelUV = frac(VoxelIndex.xyz * VoxelSize.xyz) + HalfVoxelOffset;

	// Read vector for this voxel.
	const float3 Force = Texture3DSampleLevel(VectorFieldTexture, VectorFieldTextureSampler, VoxelUV, 0).xyz;
	const float ForceScale = min(length(Force) * VectorFieldVis.Scale, 10.0f);
	const float3 Vec = normalize(Force) * ForceScale;

	// The base position for this vector.
	const FDFMatrix VolumeToWorld = MakeDFMatrix(VectorFieldVis.VolumeToWorldHigh, VectorFieldVis.RelativeVolumeToWorld);
	const float4 VolumeSpacePosition = float4(VoxelUV.xyz, 1);
	float3 BasePosition = DFTransformLocalToTranslatedWorld(VolumeSpacePosition.xyz, VolumeToWorld, ResolvedView.PreViewTranslation).xyz;
	const float3 Position = BasePosition + float(Input.VertexId) * mul(float4(Vec,0), VectorFieldVis.VolumeToWorldNoScale).xyz;

	// Build and return the set of intermediates.
	FVertexFactoryIntermediates Intermediates;
	Intermediates.SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	Intermediates.WorldPosition = float4(Position.xyz,1);
	Intermediates.PositionInstanceSpace = VolumeSpacePosition;
	return Intermediates;
}

/**
 * Computes material parameterss for a given pixel.
 * @param Interpolants - Attributes interpolated from the vertex shader.
 * @returns per-pixel material parameters.
 */
FMaterialPixelParameters GetMaterialPixelParameters( FVertexFactoryInterpolantsVSToPS Interpolants, float4 SvPosition )
{
	// GetMaterialPixelParameters is responsible for fully initializing the result
	FMaterialPixelParameters Result = MakeInitializedMaterialPixelParameters();

	// Note that a non-zero texture coordinate is used to prevent the compiler
	// from optimizing out texture lookups that can cause mandatory requirements
	// to not be bound.

#if NUM_MATERIAL_TEXCOORDS
	UNROLL
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_MATERIAL_TEXCOORDS;CoordinateIndex++)
	{
		Result.TexCoords[CoordinateIndex] = Interpolants.DummyTexCoord;
	}
#endif	//NUM_MATERIAL_TEXCOORDS

	Result.VertexColor = 1;
	Result.TwoSidedSign = 1;
	return Result;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);
float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates);

/**
 * Computes material parameters for a given vertex.
 * @param Input - Attributes for this vertex.
 * @param Intermediates - Intermediates computed for this vertex.
 * @param WorldPosition - The position of this vertex in world space.
 * @param TangentToLocal - The tangent basis for this vertex.
 * @returns per-vertex material parameters.
 */
FMaterialVertexParameters GetMaterialVertexParameters(
	FVertexFactoryInput Input, 
	FVertexFactoryIntermediates Intermediates, 
	float3 WorldPosition, 
	float3x3 TangentToLocal,
	bool bIsPreviousFrame = false)
{
	FMaterialVertexParameters Result = MakeInitializedMaterialVertexParameters();
	Result.SceneData = Intermediates.SceneData;
	
	Result.WorldPosition = WorldPosition;
	if (bIsPreviousFrame)
	{
		Result.PositionInstanceSpace = VertexFactoryGetPreviousInstanceSpacePosition(Input, Intermediates);
	}
	else
	{
		Result.PositionInstanceSpace = VertexFactoryGetInstanceSpacePosition(Input, Intermediates);
	}
	Result.PositionPrimitiveSpace = Result.PositionInstanceSpace; // No support for instancing, so instance == primitive

	Result.VertexColor = float4(1,1,1,1);
	Result.TangentToWorld = mul(TangentToLocal, GetLocalToWorld3x3()); 
	Result.PrevFrameLocalToWorld = GetPrimitiveDataFromUniformBuffer().LocalToWorld;

	Result.LWCData = MakeMaterialLWCData(Result);

	return Result;
}

/**
 * Computes the world space position of this vertex.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the position of this vertex in world space.
 */
float4 VertexFactoryGetWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.WorldPosition;
}

float3 VertexFactoryGetPreviousInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.PositionInstanceSpace.xyz;
}

float3 VertexFactoryGetWorldNormal(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return float3(0,0,1);
}

float4 VertexFactoryGetRasterizedWorldPosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float4 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

float3 VertexFactoryGetPositionForVertexLighting(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, float3 TranslatedWorldPosition)
{
	return TranslatedWorldPosition;
}

/**
 * Constructs values that need to be interpolated from the vertex shader to the pixel shader.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns interpolants.
 */
FVertexFactoryInterpolantsVSToPS VertexFactoryGetInterpolantsVSToPS( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates, FMaterialVertexParameters VertexParameters )
{
	FVertexFactoryInterpolantsVSToPS Interpolants;
	Interpolants.DummyTexCoord = float2(0,0);

	return Interpolants;
}

/**
 * Computes the position of this vertex last frame in world space.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the previous position of this vertex in world space.
 */
float4 VertexFactoryGetPreviousWorldPosition( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	return Intermediates.WorldPosition;
}

float3 VertexFactoryGetInstanceSpacePosition(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.PositionInstanceSpace.xyz;
}

/**
 * Computes the tangent basis for this vertex in world space.
 * @param Input - Vertex attributes.
 * @param Intermediates - Intermediates computed for this vertex.
 * @returns the tangent basis for this vertex in world space.
 */
float3x3 VertexFactoryGetTangentToLocal( FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates )
{
	float3x3 TangentToLocal;
	TangentToLocal[0] = float3(1,0,0);
	TangentToLocal[1] = float3(0,1,0);
	TangentToLocal[2] = float3(0,0,1);
	return TangentToLocal;
}

float4 VertexFactoryGetTranslatedPrimitiveVolumeBounds(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

uint VertexFactoryGetPrimitiveId(FVertexFactoryInterpolantsVSToPS Interpolants)
{
	return 0;
}

#include "VertexFactoryDefaultInterface.ush"