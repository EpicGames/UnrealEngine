// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VelocityCommon.usf: Common functions for calculating velocity vectors.
=============================================================================*/

#pragma once

float3 Calculate3DVelocityBase(float4 PackedVelocityA, float4 PackedVelocityC, float2 JitterA, float2 JitterC)
{
	float2 ScreenPos = PackedVelocityA.xy / PackedVelocityA.w - JitterA;
	float2 PrevScreenPos = PackedVelocityC.xy / PackedVelocityC.w - JitterC;

	float DeviceZ = PackedVelocityA.z / PackedVelocityA.w;
	float PrevDeviceZ = PackedVelocityC.z / PackedVelocityC.w;

	// 3d velocity, includes camera an object motion
	float3 Velocity = float3(ScreenPos - PrevScreenPos, DeviceZ - PrevDeviceZ);

	// Make sure not to touch 0,0 which is clear color
	return Velocity;
}

#if STEREO_MOTION_VECTORS
// If an object is rotatating too fast, the motion vector direction can get messed up and become unsuitable for OpenXR frame synthesis
// Here, when rotation speed is above a threshold (10 degrees), we'll return false and disable motion vectors by replacing PrevTransform with CurrentTransform
// To save shader performance, we only detect rotation on 2 axes
bool IsSmoothRotation( float3 PrevAxis1, float3 PrevAxis2, float3 CurrAxis1, float3 CurrAxis2 )
{
	const float angleThreshold = 0.984f; // cos(10 degree)
	float2 AngleDot = float2( dot(normalize(PrevAxis1), normalize(CurrAxis1)),
	  						  dot(normalize(PrevAxis2), normalize(CurrAxis2)));
	return all(AngleDot > angleThreshold);
}
#endif

float3 Calculate3DVelocity(float4 PackedVelocityA, float4 PackedVelocityC)
{
	return Calculate3DVelocityBase(PackedVelocityA, PackedVelocityC, ResolvedView.TemporalAAJitter.xy, ResolvedView.TemporalAAJitter.zw);
}

// Calculated the distance between packed velocity in the current frame
float3 Calculate3DVelocityOffset(float4 PackedVelocityA, float4 PackedVelocityC)
{
	return Calculate3DVelocityBase(PackedVelocityA, PackedVelocityC, ResolvedView.TemporalAAJitter.xy, ResolvedView.TemporalAAJitter.xy);
}
