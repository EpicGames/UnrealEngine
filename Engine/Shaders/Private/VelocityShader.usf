// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VelocityShader.usf: Calculates velocity vectors.
=============================================================================*/

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"
#include "VelocityCommon.ush"
#include "/Engine/Private/BasePassCommonFunctions.ush"

#ifndef VELOCITY_CLIPPED_DEPTH_PASS
#define VELOCITY_CLIPPED_DEPTH_PASS 0
#endif
// Move all geometry a little bit towards the camera to not get z fighting with existing depth
// buffer of the same meshes rendered with slightly different float computations.
// This is a positive number as our z buffer is inverted for better precision.

// 0.0f as the isolate code should do a good job to prevent this case (can be increased if that is not the case e.g. if the platform ignores the isolate)
static const float GDepthBias = 0.001f;

struct FVelocityInterpsVSToPS
{
#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH
	// float4(ScreenPos.xyzw)
	float4 PackedVelocityA : TEXCOORD6;
	// float4(PrevScreenPos.xyzw)
	float4 PackedVelocityC : TEXCOORD7;
#endif
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	float3 PixelPositionExcludingWPO : TEXCOORD8;
#endif
};

struct FVelocityVSToPS
{
	INVARIANT_OUTPUT float4 Position : SV_POSITION;
	FVelocityInterpsVSToPS Interps;
	FStereoVSOutput StereoOutput;
};

#define FVertexOutput FVelocityVSToPS
#define VertexFactoryGetInterpolants VertexFactoryGetInterpolantsVSToPS

/*=============================================================================
 * Vertex Shader
 *============================================================================*/
#if VERTEXSHADER
void MainVertexShader(
	FVertexFactoryInput Input,
	out FVertexOutput Output 
#if USE_GLOBAL_CLIP_PLANE
	, out float OutGlobalClipPlaneDistance : SV_ClipDistance
#endif
	)
{
	StereoSetupVF(Input, Output.StereoOutput);
	
	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition( Input, VFIntermediates );
#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	Output.Interps.PixelPositionExcludingWPO = TranslatedWorldPosition.xyz;
#endif
	half3x3 TangentToLocal = VertexFactoryGetTangentToLocal( Input, VFIntermediates );

	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, TranslatedWorldPosition.xyz, TangentToLocal);
	// Isolate instructions used for world position offset on xbox 360, 
	// As these cause the optimizer to generate different position calculating instructions in each pass, resulting in self-z-fighting.
	// This is only necessary for shaders used in passes that have depth testing enabled.
	{
		TranslatedWorldPosition.xyz += GetMaterialWorldPositionOffset(VertexParameters);
		ApplyMaterialFirstPersonTransform(VertexParameters, TranslatedWorldPosition.xyz);
	}

	Output.Interps.FactoryInterpolants = VertexFactoryGetInterpolants( Input, VFIntermediates, VertexParameters );

	float4 PrevTranslatedWorldPosition = VertexFactoryGetPreviousWorldPosition( Input, VFIntermediates );	
	VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, PrevTranslatedWorldPosition.xyz, TangentToLocal, true);
	PrevTranslatedWorldPosition.xyz += GetMaterialPreviousWorldPositionOffset(VertexParameters);
	ApplyMaterialPreviousFirstPersonTransform(VertexParameters, PrevTranslatedWorldPosition.xyz);
	
	{
		// compute the old screen pos with the old world position and the old camera matrix
		float4 PrevScreenPosObj = mul(float4(PrevTranslatedWorldPosition.xyz, 1), ResolvedView.PrevTranslatedWorldToClip);
		float4 ScreenPos = mul(float4(TranslatedWorldPosition.xyz, 1), ResolvedView.TranslatedWorldToClip);
		Output.Position = ScreenPos;

#if USE_GLOBAL_CLIP_PLANE
		OutGlobalClipPlaneDistance = dot(ResolvedView.GlobalClippingPlane, float4(TranslatedWorldPosition.xyz, 1));
#endif

#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH
		Output.Interps.PackedVelocityA = INVARIANT(ScreenPos);
		Output.Interps.PackedVelocityC = INVARIANT(PrevScreenPosObj);
#endif
	}

	// Move all geometry a little bit towards the camera (to not get z fighting with existing zbuffer of the same meshes rendered with slightly different float computations).
    Output.Position.z += View.NumSceneColorMSAASamples > 1 ? GDepthBias * Output.Position.w : 0.0;
#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH
	Output.Interps.PackedVelocityC.z += View.NumSceneColorMSAASamples > 1 ? GDepthBias * Output.Position.w : 0.0;
#endif
}
#endif // VERTEXSHADER

/*=============================================================================
 * Pixel Shader
 *============================================================================*/

#ifndef VELOCITY_THIN_TRANSLUCENT_MODE
#define VELOCITY_THIN_TRANSLUCENT_MODE 0
#endif

#define OUPUT_MOTION_VECTOR_WORLD_OFFSET (!TRANSLUCENCY_VELOCITY_FROM_DEPTH && !VELOCITY_CLIPPED_DEPTH_PASS && VELOCITY_SUPPORT_PS_MOTION_VECTOR_WORLD_OFFSET)

void MainPixelShader(
	in INPUT_POSITION_QUALIFIERS float4 SvPosition : SV_Position,
	FVelocityInterpsVSToPS Input,
	FStereoPSInput StereoInput
	OPTIONAL_IsFrontFace
	OPTIONAL_OutDepthConservative

	,out ENCODED_VELOCITY_TYPE OutColor : SV_Target0
#if OUPUT_MOTION_VECTOR_WORLD_OFFSET
	,out ENCODED_VELOCITY_TYPE OutMotionVectorWorldOffset : SV_Target1
#endif
#if MATERIALBLENDING_MASKED_USING_COVERAGE
	, out uint OutCoverage : SV_Coverage
#endif
	)
{
	StereoSetupPS(StereoInput);

	// Manual clipping here (alpha-test, etc)
	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Input.FactoryInterpolants, SvPosition);
	FPixelMaterialInputs PixelMaterialInputs;

	// ScreenPosition used for material calculation cannot use Input.PackedVelocityA. Using Input.PackedVelocityA 
	// will create +- 1 ULP of depth difference when compared to the logic in the basepass if PDO is use. It fails depth stencil equal test.
	float4 ScreenPosition = SvPositionToResolvedScreenPosition(SvPosition);

#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH
	float4 ScreenPositionForVelocity = Input.PackedVelocityA;
#endif

#if USE_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS && !IS_NANITE_PASS
	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(SvPosition);
	CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, SvPosition, ScreenPosition, bIsFrontFace, TranslatedWorldPosition, Input.PixelPositionExcludingWPO);
#else
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, bIsFrontFace);
#endif

#if OUTPUT_PIXEL_DEPTH_OFFSET
	float PixelDepthOffset = ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, OutDepth);

	#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH
	if (IsOrthoProjection(ResolvedView))
	{
		Input.PackedVelocityC.z += PixelDepthOffset;
		ScreenPositionForVelocity.z += PixelDepthOffset;
	}
	else
	{
		Input.PackedVelocityC.w += PixelDepthOffset;
		ScreenPositionForVelocity.w += PixelDepthOffset;
	}
	#endif

	#if APPLE_DEPTH_BIAS_HACK
	OutDepth -= APPLE_DEPTH_BIAS_VALUE;
	#endif
#endif

#if MATERIALBLENDING_MASKED_USING_COVERAGE
	OutCoverage = DiscardMaterialWithPixelCoverage(MaterialParameters, PixelMaterialInputs);
#else

#if SUBSTRATE_ENABLED && TEMPLATE_USES_SUBSTRATE
	const bool bThinTranslucentMaterial = false;

	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();

	float MaterialOpacity = 1.0f;
	if (SubstratePixelHeader.ClosureCount > 0)
	{
		// Update tree (coverage/transmittance/luminace weights)
		const FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();
		const float3 V = MaterialParameters.CameraVector;
		SubstratePixelHeader.SubstrateUpdateTree(V, Settings);

		MaterialOpacity = SubstratePixelHeader.SubstrateTree.Operators[SubstrateData.OperatorIndex].Coverage;
	}

#if SUBSTRATE_USE_PREMULTALPHA_OVERRIDE // AlphaComposite - Premultiplied alpha blending
	MaterialOpacity = GetMaterialOpacity(PixelMaterialInputs);
#endif

#else
	const bool bThinTranslucentMaterial = VELOCITY_THIN_TRANSLUCENT_MODE > 0;
	const float MaterialOpacity = GetMaterialOpacity(PixelMaterialInputs);
#endif

#if VELOCITY_CLIPPED_DEPTH_PASS && (!TRANSLUCENCY_VELOCITY_FROM_DEPTH)
	// Set a bit for those BeforeDoF translucent pixels without writing velocity
	#if MATERIALBLENDING_TRANSLUCENT || MATERIALBLENDING_ADDITIVE || MATERIALBLENDING_MODULATE || SUBSTRATE_TRANSLUCENT_MATERIAL
        #if COMPILER_SUPPORTS_UINT64_IMAGE_ATOMICS
        // We set the bit for those with Opacity value < Opacity Mask Clip Value.
		ClipLODTransition(MaterialParameters);
		float ActualMaterialOpacity = bThinTranslucentMaterial ? 1.0f : MaterialOpacity;
		const bool bWriteFlag = (ActualMaterialOpacity > 0 && ActualMaterialOpacity < (1.0 / 255.0 + GetMaterialOpacityMaskClipValue()));
		clip(select(bWriteFlag, 1.0f, -1.0f));

		uint2 Position = uint2(SvPosition.xy - 0.5f) + View.ViewRectMin.xy;
        
        // Only consider if the current translucent pixel is in front.	
		float OpaqueDepth = CalcSceneDepth(Position);

		float CurrentTranslucencyDepth = 0;
		ApplyPixelDepthOffsetToMaterialParameters(MaterialParameters, PixelMaterialInputs, CurrentTranslucencyDepth);
		CurrentTranslucencyDepth = ConvertFromDeviceZ(CurrentTranslucencyDepth); 
	
		// Use Temporal Responsiveness node to provide more flexible control.
		#if VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS
			float TranslucentTemporalResponsiveness = GetMaterialTemporalResponsiveness(MaterialParameters);
			uint TemporalResponsivenessMask = select(TranslucentTemporalResponsiveness > 0.5f, 3U, uint(TranslucentTemporalResponsiveness>0)) & TEMPORAL_RESPONSIVENESS_MASK;
		#else
			uint TemporalResponsivenessMask = 0;
		#endif
		
		uint WriteFlag = select(CurrentTranslucencyDepth < OpaqueDepth, TemporalResponsivenessMask, 0);
 
		BRANCH
		if (WriteFlag != 0)
		{ 
			const uint2 EncodedOutputBit = uint2(0u, WriteFlag << 16u); // Mask the lowest bits of alpha.
			ImageInterlockedOrUInt64(VelocityClippedDepth.RWVelocity, Position, PackUlongType(EncodedOutputBit));
		}
		#endif
	#endif
	
	clip(-1);
#else 
	GetMaterialClippingVelocity(MaterialParameters, PixelMaterialInputs, bThinTranslucentMaterial, MaterialOpacity);
#endif //VELOCITY_CLIPPED_DEPTH_PASS

#endif //MATERIALBLENDING_MASKED_USING_COVERAGE

#if !TRANSLUCENCY_VELOCITY_FROM_DEPTH

	float3 Velocity;
	float TemporalResponsiveness = GetMaterialTemporalResponsiveness(MaterialParameters);
	bool bHasPixelAnimationFlag = (GetPrimitiveData(MaterialParameters).Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_PIXEL_ANIMATION) != 0;
 
	#if (!VELOCITY_CLIPPED_DEPTH_PASS && VELOCITY_SUPPORT_PS_MOTION_VECTOR_WORLD_OFFSET) // Opaque pass that uses motion vector offset in PS.
		
		float3 OffsetTranslatedWorldPosition = GetMaterialMotionVectorWorldOffset(MaterialParameters) + GetTranslatedWorldPosition(MaterialParameters);
		float4 ClipPosition = mul(float4(OffsetTranslatedWorldPosition, 1.0f), ResolvedView.TranslatedWorldToClip);

		#if OUTPUT_PIXEL_DEPTH_OFFSET
		if (IsOrthoProjection(ResolvedView))
		{
			ClipPosition.z += PixelDepthOffset;
		}
		else
		{
			ClipPosition.w += PixelDepthOffset;
		}
		#endif

		{
			// Write the offset at the same time so that we can recover the actual velocity to previous frame.
			OutMotionVectorWorldOffset = EncodeVelocityToTexture(Calculate3DVelocityOffset(ScreenPositionForVelocity, ClipPosition), bHasPixelAnimationFlag, TemporalResponsiveness);

			Velocity = Calculate3DVelocity(ScreenPositionForVelocity, Input.PackedVelocityC);
		}
	#else	
		Velocity = Calculate3DVelocity(ScreenPositionForVelocity, Input.PackedVelocityC);
	#endif // !VELOCITY_CLIPPED_DEPTH_PASS

#if STEREO_MOTION_VECTORS
	OutColor.xyz = Velocity;
	OutColor.w = 0.0f;
#else
	OutColor = EncodeVelocityToTexture(Velocity, bHasPixelAnimationFlag, TemporalResponsiveness);
#endif
#else
	// We only want velocity generated from camera movement and depth so write zero which is the special clear color value.
	OutColor = 0;
#endif 
}
