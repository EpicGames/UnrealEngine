// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VelocityUpdate.usf: Make any update to velocity vectors.
=============================================================================*/

#include "Common.ush"
#include "VelocityCommon.ush"
#include "SceneTexturesCommon.ush"

Texture2D<float4> DepthTexture;
Texture2D<ENCODED_VELOCITY_TYPE> VelocityTexture;
#if SLW_RESOLVE
Texture2D<ENCODED_VELOCITY_TYPE> MotionVectorWorldOffset;
RWTexture2D<ENCODED_VELOCITY_TYPE> RWOutputVelocity;
#else
// Note: Despite the name, this is actually the output velocity texture (containing the offset).
// The caller swaps the velocity and offset bindings so that the resolve CS can write the merged
// result directly into the velocity texture without an additional copy. See RenderVelocities().
RWTexture2D<ENCODED_VELOCITY_TYPE> RWMotionVectorWorldOffset;
#endif

bool PointInView(float2 Position)
{
	return (all(Position >= ResolvedView.ViewRectMin.xy) &&  
			all(Position < ResolvedView.ViewRectMin.xy + ResolvedView.ViewSizeAndInvSize.xy));
}

float3 DecodeVelocityTexture(Texture2D<ENCODED_VELOCITY_TYPE> Velocity, float2 Position)
{
	int2 IntPosition = int2(Position);
	//TODO: sub-pixel hole filing.
	ENCODED_VELOCITY_TYPE EncodedVelocity = Velocity[IntPosition + int2(0, 0)];
	return EncodedVelocity.x > 0 ? DecodeVelocityFromTexture(EncodedVelocity) : 0;
}

float CalcDeviceZ(float2 Position)
{
	int2 IntPosition = int2(Position);
	#if 1
		return DepthTexture.Load(int3(IntPosition, 0)).r;
	#else
		float2 Weight = frac(Position);
		float P00 = DepthTexture.Load(int3(IntPosition + int2(0, 0), 0)).r;
		float P01 = DepthTexture.Load(int3(IntPosition + int2(0, 1), 0)).r;
		float P10 = DepthTexture.Load(int3(IntPosition + int2(1, 0), 0)).r;
		float P11 = DepthTexture.Load(int3(IntPosition + int2(1, 1), 0)).r;
		return lerp(lerp(P00, P10, Weight.x),lerp(P01,P11, Weight.x),Weight.y);
	#endif
}

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void MainCS(in uint3 ThreadID : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();
	int2 Position = ThreadID.xy;
	if (any(Position >= ResolvedView.ViewSizeAndInvSize.xy))
	{
		return;
	}
	int2 TexelPosition = Position + ResolvedView.ViewRectMin.xy;
 	
#if SLW_RESOLVE
	ENCODED_VELOCITY_TYPE EncodedVectorOffset = MotionVectorWorldOffset[TexelPosition];
#else
	ENCODED_VELOCITY_TYPE EncodedVectorOffset = RWMotionVectorWorldOffset[TexelPosition];
#endif
	uint TemporalResponsivenessMask = 0;
	#if (VELOCITY_ENCODE_HAS_PIXEL_ANIMATION | VELOCITY_ENCODE_TEMPORAL_RESPONSIVENESS)	
		TemporalResponsivenessMask |= (uint(round(EncodedVectorOffset.w * 65535.0f)) & TEMPORAL_RESPONSIVENESS_MASK);
	#endif
	float3 DecodedVectorOffset = DecodeVelocityFromTexture(EncodedVectorOffset);
	bool bHasVelocity = EncodedVectorOffset.x > 0;
	float2 OffsetTexelPosition = TexelPosition - (ScreenPosToViewportUV(DecodedVectorOffset.xy) - 0.5f/*remove bias*/) * ResolvedView.ViewSizeAndInvSize.xy;// non-integer velocity;
	bHasVelocity &= any(OffsetTexelPosition != float2(TexelPosition));

#if SLW_RESOLVE
	// Preserve existing opaque velocity for non-SLW pixels.
	if (!bHasVelocity)
	{
		return;
	}
#endif

	ENCODED_VELOCITY_TYPE EncodedMergedVelocity;
	if (bHasVelocity)
	{
		// Velocity at border that projects to outside of the view uses the current frame's offset and velocity.
		float3 DecodedTargetVelocity = DecodeVelocityTexture(VelocityTexture, PointInView(OffsetTexelPosition) ? OffsetTexelPosition : TexelPosition);
		float2 TargetTexelPosition = OffsetTexelPosition - (ScreenPosToViewportUV(DecodedTargetVelocity.xy) - 0.5f)*ResolvedView.ViewSizeAndInvSize.xy;
	
		// Construct velocity offset from Position to Target Texel Position.
		float SourceDeviceZ = DepthTexture.Load(int3(TexelPosition, 0)).r;
		float OffsetDeviceZ = CalcDeviceZ(OffsetTexelPosition);
		float3 Velocity = float3(DecodedVectorOffset.xy, SourceDeviceZ - OffsetDeviceZ) + DecodedTargetVelocity;

		//TODO: check if the TargetTexelPosition is within bound for rigid body transform to reject history that is too far/close.
		//  |QtQt+1| \in (|PtPt+1| +/- (s+1)/s * |Pt+1Qt+1|, where s is the scaling.
		EncodedMergedVelocity = EncodeVelocityToTexture(Velocity,TemporalResponsivenessMask);
	}
	else
	{
		EncodedMergedVelocity = VelocityTexture[TexelPosition];
	}

#if SLW_RESOLVE
	RWOutputVelocity[TexelPosition] = EncodedMergedVelocity;
#else
	RWMotionVectorWorldOffset[TexelPosition] = EncodedMergedVelocity;
#endif
}

