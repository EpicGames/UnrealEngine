// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SceneData.ush"

//
// Functions to get instanced stereo properties from a VF. They have to be macros as hlslcc does not support member functions (as of May 2022 it is still used for a couple platforms),
// whereas defining them as overloaded functions (taking VF as parameter) doesn't work as some VFs can be identical for typeless HLSL overloading rules.
//
#if INSTANCED_STEREO
	#define GetInstanceIdFromVF(VFInput)				(VFInput.InstanceId)
	#define GetEyeIndexFromVF(VFInput)					(IsInstancedStereo() ? (GetInstanceIdFromVF(VFInput) & 1) : 0)
#elif MOBILE_MULTI_VIEW
	#define GetInstanceIdFromVF(VFInput)				(0)
	#define GetEyeIndexFromVF(VFInput)					(VFInput.ViewId)
#else
	#define GetInstanceIdFromVF(VFInput)				(0)
	#define GetEyeIndexFromVF(VFInput)					(0)
#endif

#if MOBILE_MULTI_VIEW && !INSTANCED_STEREO
    #define VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK() \
        uint ViewId : SV_ViewID;
#else
	#define VF_MOBILE_MULTI_VIEW_DECLARE_INPUT_BLOCK()
#endif // MOBILE_MULTI_VIEW && !INSTANCED_STEREO

//  
// Required VF inputs to support dynamic instancing 
//
#if VF_USE_PRIMITIVE_SCENE_DATA
	#if SHADER_USES_PRIMITIVE_UBO
		#define VF_GPUSCENE_DECLARE_INPUT_BLOCK(Attr0) \
			uint DrawInstanceId : SV_InstanceID;

		#define VF_GPUSCENE_GET_INTERMEDIATES(VFInput) \
			GetSceneDataIntermediates(0u, VFInput.DrawInstanceId)
	#else
		#define VF_GPUSCENE_DECLARE_INPUT_BLOCK(Attr0) \
			uint InstanceIdOffset : ATTRIBUTE##Attr0; \
			uint DrawInstanceId : SV_InstanceID;

		#define VF_GPUSCENE_GET_INTERMEDIATES(VFInput) \
			GetSceneDataIntermediates(VFInput.InstanceIdOffset, VFInput.DrawInstanceId)
	#endif //SHADER_USES_PRIMITIVE_UBO
	
	#define VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(VFInput, LightmapDataIndex) \
		GetLightmapData(LightmapDataIndex, VFInput.DrawInstanceId).LightMapCoordinateScaleBias
	
	#define VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(VFInput, LightmapDataIndex) \
		GetLightmapData(LightmapDataIndex, VFInput.DrawInstanceId).ShadowMapCoordinateScaleBias

	#define VF_GPUSCENE_SET_INPUT_FOR_RT(_VFInputOut_, _InstanceId_, _DrawInstanceId_) \
		_VFInputOut_.InstanceIdOffset = _InstanceId_ - _DrawInstanceId_; \
		_VFInputOut_.DrawInstanceId = _DrawInstanceId_;

	// GPUCULL_TODO: Makes use of DynamicDraw workaround, should instead refactor VF interfaces and provide the instance ID data directly.
	#define VF_GPUSCENE_SET_INPUT_FOR_HAIR(_VFInputOut_, _PrimitiveId_, _DrawInstanceId_) \
		_VFInputOut_.InstanceIdOffset = (_PrimitiveId_) | VF_TREAT_INSTANCE_ID_OFFSET_AS_PRIMITIVE_ID_FLAG;	\
		_VFInputOut_.DrawInstanceId = _DrawInstanceId_;

	#define VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()

	#undef GetInstanceIdFromVF
	#define GetInstanceIdFromVF(VFInput)				(VFInput.DrawInstanceId)
#else
	#define VF_GPUSCENE_DECLARE_INPUT_BLOCK(Attr0) 
	#define VF_GPUSCENE_GET_INTERMEDIATES(VFInput) GetSceneDataIntermediates()
	#define VF_GPUSCENE_GET_LIGHTMAP_UV_SCALE_BIAS(VFInput, LightmapDataIndex) PrecomputedLightingBuffer.LightMapCoordinateScaleBias
	#define VF_GPUSCENE_GET_SHADOWMAP_UV_SCALE_BIAS(VFInput, LightmapDataIndex) PrecomputedLightingBuffer.ShadowMapCoordinateScaleBias
	#define VF_GPUSCENE_SET_INPUT_FOR_RT(_VFInputOut_, _InstanceId_, _DrawInstanceId_)
	#define VF_GPUSCENE_SET_INPUT_FOR_HAIR(_VFInputOut_, _PrimitiveId_, _DrawInstanceId_)

	#if INSTANCED_STEREO
		#define VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK() \
			uint InstanceId : SV_InstanceID;
	#else
		#define VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK()
	#endif 

#endif

// In case other paths in the above #if miss the declaration, catch this early
#if !defined(VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK)
	#error VF_INSTANCED_STEREO_DECLARE_INPUT_BLOCK() should be defined for a vertex factory
#endif

#if !defined(GetInstanceIdFromVF)
	#error GetInstanceIdFromVF() should be defined.
#endif

#if !defined(GetEyeIndexFromVF)
	#error GetEyeIndexFromVF() should be defined.
#endif

#if INSTANCED_STEREO
    #define StereoSetupVF(VFInput, Output)			StereoSetupVS((IsInstancedStereo() ? (GetInstanceIdFromVF(VFInput) & 1) : 0), Output);
#elif MOBILE_MULTI_VIEW
    #define StereoSetupVF(VFInput, Output)			StereoSetupVS(VFInput.ViewId, Output);
#else
    #define StereoSetupVF(VFInput, Output)			StereoSetupVS(0, Output); // This will resolve the view regardless of the EyeIndex
#endif

FDFVector3 TransformLocalToWorld(float3 LocalPosition, FDFMatrix LocalToWorld)
{
	return DFMultiplyAffine(LocalPosition, LocalToWorld);
}

FDFVector3 TransformLocalToWorld(float3 LocalPosition)
{
	return TransformLocalToWorld(LocalPosition, GetPrimitiveDataFromUniformBuffer().LocalToWorld);
}

float4 TransformLocalToTranslatedWorld(float3 LocalPosition, FDFMatrix LocalToWorld)
{
	return DFTransformLocalToTranslatedWorld(LocalPosition, LocalToWorld, ResolvedView.PreViewTranslation);
}

float4 TransformLocalToTranslatedWorld(float3 LocalPosition, float4x4 LocalToWorld)
{
	return TransformLocalToTranslatedWorld(LocalPosition, DFPromote(LocalToWorld));
}

float4 TransformPreviousLocalPositionToTranslatedWorld(float3 PrevLocalPosition, FDFMatrix PrevLocalToWorld)
{
	return DFTransformLocalToTranslatedWorld(PrevLocalPosition, PrevLocalToWorld, ResolvedView.PrevPreViewTranslation);
}

float4 TransformPreviousLocalPositionToTranslatedWorld(float3 PrevLocalPosition, float4x4 PrevLocalToWorld)
{
	return TransformPreviousLocalPositionToTranslatedWorld(PrevLocalPosition, DFPromote(PrevLocalToWorld));
}

float3 RotateLocalToWorld(float3 LocalDirection, float3x3 LocalToWorld, float3 InvScale)
{
	return
		InvScale.x * LocalToWorld[0] * LocalDirection.xxx +
		InvScale.y * LocalToWorld[1] * LocalDirection.yyy +
		InvScale.z * LocalToWorld[2] * LocalDirection.zzz;
}

float3 RotateLocalToWorld(float3 LocalDirection, float4x4 LocalToWorld, float3 InvScale)
{
	return RotateLocalToWorld(LocalDirection, (float3x3)LocalToWorld, InvScale);
}

float3 RotateLocalToWorld(float3 LocalDirection, FDFMatrix LocalToWorld, float3 InvScale)
{
	return RotateLocalToWorld(LocalDirection, DFToFloat3x3(LocalToWorld), InvScale);
}

#if !VF_USE_PRIMITIVE_SCENE_DATA

float4 TransformLocalToTranslatedWorld(float3 LocalPosition)
{
	return TransformLocalToTranslatedWorld(LocalPosition, GetPrimitiveDataFromUniformBuffer().LocalToWorld);
}

float3 RotateLocalToWorld(float3 LocalDirection)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveDataFromUniformBuffer();
	return RotateLocalToWorld(LocalDirection, PrimitiveData.LocalToWorld, PrimitiveData.InvNonUniformScale);
}

float3 RotateWorldToLocal(float3 WorldDirection)
{
	FPrimitiveSceneData PrimitiveData = GetPrimitiveDataFromUniformBuffer();
	const float3 InvScale = PrimitiveData.InvNonUniformScale;
	const float3x3 LocalToWorld = DFToFloat3x3(PrimitiveData.LocalToWorld);
	float3x3 InvRot = {
		InvScale.x * LocalToWorld[0],
		InvScale.y * LocalToWorld[1],
		InvScale.z * LocalToWorld[2]
	};
	InvRot = transpose(InvRot);
	return mul(WorldDirection, InvRot);
}

#endif // !VF_USE_PRIMITIVE_SCENE_DATA

// Octahedron Normal Vectors
// [Cigolle 2014, "A Survey of Efficient Representations for Independent Unit Vectors"]
//						Mean	Max
// oct		8:8			0.33709 0.94424
// snorm	8:8:8		0.17015 0.38588
// oct		10:10		0.08380 0.23467
// snorm	10:10:10	0.04228 0.09598
// oct		12:12		0.02091 0.05874

float2 UnitToOct( float3 N )
{
	N.xy /= dot( 1, abs(N) );
	if( N.z <= 0 )
	{
		N.xy = ( 1 - abs(N.yx) ) * select( N.xy >= 0, float2(1,1), float2(-1,-1) );
	}
	return N.xy;
}

float3 OctToUnit( float2 Oct )
{
	float3 N = float3( Oct, 1 - dot( 1, abs(Oct) ) );
	if( N.z < 0 )
	{
		N.xy = ( 1 - abs(N.yx) ) * select( N.xy >= 0, float2(1,1), float2(-1,-1) );
	}
	return normalize(N);
}

#if RAYHITGROUPSHADER && !ALLOW_CUSTOM_RAYTRACING_HIT_TYPE
bool IsFrontFaceHit()
{
    return HitKind() == HIT_KIND_TRIANGLE_FRONT_FACE;
}
#endif