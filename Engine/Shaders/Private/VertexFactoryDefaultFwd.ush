// Copyright Epic Games, Inc. All Rights Reserved.

// default/empty implementation of VertexFactoryFwd