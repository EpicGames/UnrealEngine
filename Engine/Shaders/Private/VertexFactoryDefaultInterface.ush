// Copyright Epic Games, Inc. All Rights Reserved.

// Note: include at the end of vertex factories to provide default/dummy implementation of factory functions that are not supported OR that follow the standard pattern
// E.g., if the VF implements the three flavours of VertexFactoryGetViewIndex, it should define VF_IMPLEMENTED_GET_VIEW_INDEX

#ifndef VF_IMPLEMENTED_GET_SCENE_DATA_INTERMEDIATES
#define VF_IMPLEMENTED_GET_SCENE_DATA_INTERMEDIATES

FSceneDataIntermediates GetSceneDataIntermediates(FVertexFactoryIntermediates Intermediates)
{
	return Intermediates.SceneData;
}

#endif


#ifndef VF_IMPLEMENTED_GET_VIEW_INDEX
#define VF_IMPLEMENTED_GET_VIEW_INDEX

uint VertexFactoryGetViewIndex(FVertexFactoryIntermediates Intermediates)
{
	return GetSceneDataIntermediates(Intermediates).ViewIndex;
}

uint VertexFactoryGetInstanceIdLoadIndex(FVertexFactoryIntermediates Intermediates)
{
	return GetSceneDataIntermediates(Intermediates).InstanceIdLoadIndex;
}

FDFMatrix VertexFactoryGetLocalToWorld(FVertexFactoryIntermediates Intermediates)
{
	return GetSceneDataIntermediates(Intermediates).InstanceData.LocalToWorld;
}

FDFInverseMatrix VertexFactoryGetWorldToLocal(FVertexFactoryIntermediates Intermediates)
{
	return GetSceneDataIntermediates(Intermediates).InstanceData.WorldToLocal;
}

#if POSITION_ONLY

uint VertexFactoryGetViewIndex(FPositionOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return SceneData.ViewIndex;
}

uint VertexFactoryGetViewIndex(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return SceneData.ViewIndex;
}

uint VertexFactoryGetInstanceIdLoadIndex(FPositionOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return SceneData.InstanceIdLoadIndex;
}

uint VertexFactoryGetInstanceIdLoadIndex(FPositionAndNormalOnlyVertexFactoryInput Input)
{
	FSceneDataIntermediates SceneData = VF_GPUSCENE_GET_INTERMEDIATES(Input);
	return SceneData.InstanceIdLoadIndex;
}

#endif // POSITION_ONLY

#endif // VF_IMPLEMENTED_GET_VIEW_INDEX

#ifndef VF_IMPLEMENTED_GET_INSTANCE_HIT_PROXY_ID
#define VF_IMPLEMENTED_GET_INSTANCE_HIT_PROXY_ID

float4 VertexFactoryGetInstanceHitProxyId(FVertexFactoryInput Input, FVertexFactoryIntermediates Intermediates)
{
#if USE_EDITOR_SHADERS
	return float4(GetSceneDataIntermediates(Intermediates).InstanceData.EditorData.HitProxyId, 0.0f);
#else
	return float4(0.0f, 0.0f, 0.0f, 0.0f);
#endif
}

#endif // VF_IMPLEMENTED_GET_INSTANCE_HIT_PROXY_ID
