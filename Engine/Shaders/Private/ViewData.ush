// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "WaveOpUtil.ush"

#include "/Engine/Shared/ViewDefinitions.h"

// define the default access mode to be via the Scene UB
#ifndef VIEW_DATA_ACCESS_MODE
#define VIEW_DATA_ACCESS_MODE VIEW_DATA_ACCESS_SCENE_UB
#endif

// API for shaders that modify the view data.
#if VIEW_DATA_ACCESS_MODE == VIEW_DATA_ACCESS_RW 

int MaxPersistentViewId;
uint InstanceMaskWordStride;
uint NumSceneRendererPrimaryViews;
RWStructuredBuffer<uint> OutDeformingInstanceViewMask;
StructuredBuffer<int> PersistentIdToIndexMap;

void MarkInstanceAsDeforming(uint InstanceId, uint ViewIndex)
{
	uint Mask = 1u << (InstanceId % 32u);
	uint MaskWordOffset = InstanceId / 32u;
	// Mark for the particular view
	InterlockedOr(OutDeformingInstanceViewMask[(InstanceMaskWordStride * ViewIndex) + MaskWordOffset], Mask);
}

void MarkInstanceAsDeformingWithViewMask(uint InstanceId, uint ViewMask)
{
	uint Mask = 1u << (InstanceId % 32u);
	uint MaskWordOffset = InstanceId / 32u;

	if (ViewMask != 0u)
	{
		// Mark for the view in the mask
		for (uint ViewIndex = 0u; ViewIndex < NumSceneRendererPrimaryViews; ++ViewIndex)
		{
			if (((1u << ViewIndex) & ViewMask) !=  0u)
			{
				InterlockedOr(OutDeformingInstanceViewMask[(InstanceMaskWordStride * ViewIndex) + MaskWordOffset], Mask);
			}
		}
	}
}

#endif 

struct FViewData
{
	int MaxPersistentViewId;
	uint InstanceMaskWordStride;
	uint NumSceneRendererPrimaryViews;
};

FViewData GetViewData()
{
	FViewData ViewData;
#if VIEW_DATA_ACCESS_MODE == VIEW_DATA_ACCESS_RW 
	ViewData.MaxPersistentViewId = MaxPersistentViewId;
	ViewData.InstanceMaskWordStride = InstanceMaskWordStride;
	ViewData.NumSceneRendererPrimaryViews = NumSceneRendererPrimaryViews;
#else
	ViewData.MaxPersistentViewId = Scene.ViewData.MaxPersistentViewId;
	ViewData.InstanceMaskWordStride = Scene.ViewData.InstanceMaskWordStride;
	ViewData.NumSceneRendererPrimaryViews = Scene.ViewData.NumSceneRendererPrimaryViews;
#endif
	return ViewData;
}

/**
 * Returns a 32-bit word with instance-deforming flags for 32 consecutive instances starting from InstanceId = MaskWordOffset * 32
 */
uint LoadInstanceDeformingWord(uint MaskWordOffset, int ViewIndex)
{
	uint BaseOffset = ViewIndex * GetViewData().InstanceMaskWordStride;
#if VIEW_DATA_ACCESS_MODE == VIEW_DATA_ACCESS_RW 
	return OutDeformingInstanceViewMask[BaseOffset + MaskWordOffset];
#else // VIEW_DATA_ACCESS_SCENE_UB
	return Scene.ViewData.DeformingInstanceViewMask[BaseOffset + MaskWordOffset];
#endif
}

/**
 * Returns true if the instance is marked as deforming. 
 */
bool IsInstanceDeforming(uint InstanceId, int SceneRendererPrimaryViewId)
{
	uint Mask = 1u << (InstanceId % 32u);
	uint MaskWordOffset = InstanceId / 32u;
	return (LoadInstanceDeformingWord(MaskWordOffset, SceneRendererPrimaryViewId) & Mask) != 0u;
}

/**
 * Map a persistent view ID to a current SceneRendererPrimaryViewId, returns -1 if the view is not current (part of the ongoing scenerenderer)
 */
int RemapPersisitentViewId(int PersistentViewId)
{
	if (PersistentViewId < GetViewData().MaxPersistentViewId)
	{
#if VIEW_DATA_ACCESS_MODE == VIEW_DATA_ACCESS_RW 
		return int(PersistentIdToIndexMap[PersistentViewId]) - 1;
#else
		return int(Scene.ViewData.PersistentIdToIndexMap[PersistentViewId]) - 1;
#endif
	}
	return -1;
}

/**
 */
struct FInstanceViewData
{
	bool bIsValid;
	bool bIsDeforming;

	// Convenience / derived data 
	uint InstanceId;
	int SceneRendererPrimaryViewId;
};

FInstanceViewData GetInstanceViewData(uint InstanceId, int SceneRendererPrimaryViewId)
{
	checkSlow(SceneRendererPrimaryViewId < GetViewData().NumSceneRendererPrimaryViews && SceneRendererPrimaryViewId >= 0);

	FInstanceViewData InstanceViewData;
	
	// Convenience / derived data 
	InstanceViewData.SceneRendererPrimaryViewId = SceneRendererPrimaryViewId;
	InstanceViewData.InstanceId = InstanceId;
	
	InstanceViewData.bIsValid = true;

	InstanceViewData.bIsDeforming = IsInstanceDeforming(InstanceId, SceneRendererPrimaryViewId);
	return InstanceViewData;
}
