// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

Texture2D InputTexture;
SamplerState InputSampler;

float Grayscale(float3 SourceClr)
{
	return 0.299 * SourceClr.r + 0.587 * SourceClr.g + 0.114 * SourceClr.b;
}

float4 DesaturatePS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
) : SV_Target0
{
	float2 UV = UVAndScreenPos.xy;
	float4 Color = Texture2DSample(InputTexture, InputSampler, UV);
	Color.rgb = Grayscale(Color.rgb);
	return Color;
}
