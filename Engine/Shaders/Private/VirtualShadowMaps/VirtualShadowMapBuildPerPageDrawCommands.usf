// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapBuildPerPageDrawCommands.usf: 
=============================================================================*/

// Do not use shared samplers as it requires the View uniform buffer, which is not bound for this shader.
#define USE_HZB_SHARED_SAMPLERS 0

#include "../Common.ush"
#include "../ViewData.ush"
#include "VirtualShadowMapPageOverlap.ush"
#include "VirtualShadowMapProjectionCommon.ush"

#define VIRTUAL_TEXTURE_TARGET 1

// Turn on the logic for culling based on min screen radius (used in NaniteCullingCommon.ush)
#define NANITE_CULLING_ENABLE_MIN_RADIUS_CULL 1

// Turn on various VSM-related logic in NaniteCullingCommon
#define VIRTUAL_TEXTURE_TARGET 1

#include "../Nanite/NaniteCullingCommon.ush"
#include "../Nanite/NaniteDataDecode.ush"
#include "../InstanceCulling/InstanceCullingCommon.ush"
#include "../InstanceCulling/InstanceCullingLoadBalancer.ush"
#include "../WaveOpUtil.ush"
#include "../DynamicMeshBounds.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "VirtualShadowMapStats.ush"
#include "VirtualShadowMapPageCacheCommon.ush"

#include "../ShaderPrint.ush"

RWStructuredBuffer<FVSMVisibleInstanceCmd> VisibleInstancesOut;
RWStructuredBuffer<uint> VisibleInstanceCountBufferOut;
uint TotalPrimaryViews;
uint VisibleInstancesBufferNum;

#if ENABLE_BATCH_MODE
StructuredBuffer<FVSMCullingBatchInfo> VSMCullingBatchInfos;
#else // !ENABLE_BATCH_MODE
uint FirstPrimaryView;
uint NumPrimaryViews;
uint DynamicInstanceIdOffset;
uint DynamicInstanceIdMax;
#endif // ENABLE_BATCH_MODE

StructuredBuffer<uint2> DrawCommandDescs;

RWBuffer<uint> DrawIndirectArgsBufferOut;

// TODO: Move to common header
// Get the area of an "inclusive" rect (which means that the max is inside the rect), also guards against negative area (where min > max)
uint GetInclusiveRectArea(uint4 Rect)
{
	if (all(Rect.zw >= Rect.xy))
	{
		uint2 Size = Rect.zw - Rect.xy;
		return (Size.x + 1) * (Size.y + 1);
	}
	return 0;
}

bool WriteCmd(uint MipViewId, uint InstanceId, uint IndirectArgIndex, uint CullingFlags, bool bStaticPage)
{
	FPageInfo PageInfo;
	PageInfo.ViewId = MipViewId;
	PageInfo.bStaticPage = bStaticPage;

	FVSMVisibleInstanceCmd VisibleInstanceCmd;
	VisibleInstanceCmd.PackedPageInfo = PackPageInfo(PageInfo);
	VisibleInstanceCmd.InstanceIdAndFlags = (CullingFlags << INSTANCE_ID_NUM_BITS) | InstanceId;
	VisibleInstanceCmd.IndirectArgIndex = IndirectArgIndex;

	uint VisibleInstanceOutputOffset = 0U;
	WaveInterlockedAddScalar_(VisibleInstanceCountBufferOut[0], 1U, VisibleInstanceOutputOffset);
	if (VisibleInstanceOutputOffset < VisibleInstancesBufferNum)
	{
		VisibleInstancesOut[VisibleInstanceOutputOffset] = VisibleInstanceCmd;
		return true;
	}
	else
	{
		return false;
	}
}

#define MAX_SINGLE_THREAD_MARKING_AREA (8U)
#define MARKING_JOB_QUEUE_SIZE (NUM_THREADS_PER_GROUP * 2U)

struct FSharedMarkingJob
{
	uint4 RectPages;
	uint VirtualShadowMapId;
	uint MipLevel;
	uint FlagMask;
	uint MarkPageDirtyFlags;
};

groupshared uint NumSharedMarkingJobs;
groupshared uint2 SharedMarkingJobs[MARKING_JOB_QUEUE_SIZE];

uint2 PackJob(FSharedMarkingJob Job)
{
	uint2 Packed;
	Packed.x = (Job.RectPages.x << 25U) 
		| (Job.RectPages.y << 18U) 
		| (Job.RectPages.z << 11U) 
		| (Job.RectPages.w << 4u)
		| Job.FlagMask;
	Packed.y = (Job.VirtualShadowMapId << 10U) 
		| (Job.MipLevel << 6U) 
		| (Job.MarkPageDirtyFlags & 0x3F);
	return Packed;
}

FSharedMarkingJob UnpackJob(uint2 Packed)
{
	FSharedMarkingJob Job;
	Job.RectPages.x = Packed.x >> 25u;
	Job.RectPages.y = (Packed.x >> 18u) & 0x7Fu;
	Job.RectPages.z = (Packed.x >> 11u) & 0x7Fu;
	Job.RectPages.w = (Packed.x >> 4u) &  0x7Fu;
	Job.FlagMask = Packed.x & 0xFu;
	Job.VirtualShadowMapId = Packed.y >> 10U;
	Job.MipLevel = (Packed.y >> 6u) & 0x7;
	Job.MarkPageDirtyFlags = (Packed.y & 0x3F);
	return Job;
}

bool MarkPageDirty(
	FVirtualSMLevelOffset PageTableLevelOffset,
	uint2 vPage,
	uint MipLevel,
	uint PageFlagMask,
	uint MarkPageDirtyFlags)
{
	// TODO: Do we actually even need this check?
	FVSMPageOffset PageFlagOffset = CalcPageOffset(PageTableLevelOffset, MipLevel, vPage);
	uint PageFlag = VirtualShadowMapGetPageFlag(PageFlagOffset);
	if ((PageFlag & PageFlagMask) != 0)
	{
		return VirtualShadowMapMarkPageDirty(PageFlagOffset, MarkPageDirtyFlags);
	}
	return false;
}

#if VSM_GENERATE_STATS
uint NumPageAreaDiagnosticSlots;
uint LargeInstancePageAreaThreshold;
#endif

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void CullPerPageDrawCommandsCs(uint3 GroupId : SV_GroupID, int GroupThreadIndex : SV_GroupIndex)
{
	uint DispatchGroupId = GetUnWrappedDispatchGroupId(GroupId);

	if (DispatchGroupId >= InstanceCullingLoadBalancer_GetNumBatches())
	{
		return;
	}
	
	if (GroupThreadIndex == 0)
	{
		NumSharedMarkingJobs = 0U;
	}
	GroupMemoryBarrierWithGroupSync();

#if ENABLE_BATCH_MODE
	// Load Instance culling context batch info, indirection per group
	FContextBatchInfo BatchInfo = LoadBatchInfo(DispatchGroupId);
	FVSMCullingBatchInfo VSMCullingBatchInfo = VSMCullingBatchInfos[BatchInds[DispatchGroupId]];
#else // !ENABLE_BATCH_MODE
	// Single Instance culling context batch in the call, set up batch from the kernel parameters
	FContextBatchInfo BatchInfo = (FContextBatchInfo)0;
	BatchInfo.DynamicInstanceIdOffset = DynamicInstanceIdOffset;
	BatchInfo.DynamicInstanceIdMax = DynamicInstanceIdMax;
	// Note: for the unbatched case, the permutation will control HZB test, so we set to true
	BatchInfo.bAllowOcclusionCulling = true;

	FVSMCullingBatchInfo VSMCullingBatchInfo;
	VSMCullingBatchInfo.FirstPrimaryView = FirstPrimaryView;
	VSMCullingBatchInfo.NumPrimaryViews = NumPrimaryViews;

#endif // ENABLE_BATCH_MODE

	uint CurrentBatchProcessingMode = 0U;
	FInstanceWorkSetup WorkSetup = InstanceCullingLoadBalancer_Setup(GroupId, GroupThreadIndex, GetItemDataOffset(BatchInfo, CurrentBatchProcessingMode));
	if (WorkSetup.bValid)
	{
		const FInstanceCullingPayload Payload = LoadInstanceCullingPayload(WorkSetup.Item.Payload, BatchInfo);

		uint InstanceDataOffset = WorkSetup.Item.InstanceDataOffset;

		if (Payload.bDynamicInstanceDataOffset)
		{
			InstanceDataOffset += BatchInfo.DynamicInstanceIdOffset;
			checkSlow(InstanceDataOffset + uint(WorkSetup.LocalItemIndex) < BatchInfo.DynamicInstanceIdMax);
		}

		uint InstanceId = InstanceDataOffset + uint(WorkSetup.LocalItemIndex);
		FDrawCommandDesc DrawCommandDesc = UnpackDrawCommandDesc(DrawCommandDescs[Payload.IndirectArgIndex]);

		// Load relevant instance data
		FInstanceSceneData InstanceData = GetInstanceSceneData(InstanceId);
		FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);

		float3 LocalBoundsCenter = InstanceData.LocalBoundsCenter;
		float3 LocalBoundsExtent = InstanceData.LocalBoundsExtent;
		LoadDynamicMeshBounds(DrawCommandDesc.DynamicMeshBoundsIndex, LocalBoundsCenter, LocalBoundsExtent);

#if WITH_SHADERPRINT
		float4 BoundsColor = all(LocalBoundsCenter == InstanceData.LocalBoundsCenter) ? ColorRed : ColorGreen;
		AddOBBWS(LocalBoundsCenter - LocalBoundsExtent, LocalBoundsCenter + LocalBoundsExtent, BoundsColor, DFHackToFloat(InstanceData.LocalToWorld));
#endif

		uint ThreadTotalForAllViews = 0;
#if VSM_GENERATE_STATS
		uint TotalPageArea = 0U;
		float NumVisibleLevels = 0.0f;
#endif
		bool bMarkingJobQueueOverflow = false;
		bool bVisibleInstancesOverflow = false;

		// Loop over views and output visible instance (i.e., those that overlap a valid page)
		for (uint PrimaryViewId = VSMCullingBatchInfo.FirstPrimaryView; PrimaryViewId < VSMCullingBatchInfo.FirstPrimaryView + VSMCullingBatchInfo.NumPrimaryViews; ++PrimaryViewId)
		{
			FNaniteView NaniteView = GetNaniteView(PrimaryViewId);

			uint CullingFlags = INSTANCE_CULLING_FLAGS_DEFAULT;
			bool bEnableWPO = DrawCommandDesc.bMaterialUsesWorldPositionOffset;
		
			FInstanceDynamicData DynamicData = CalculateInstanceDynamicData(NaniteView, InstanceData, true);

			FBoxCull Cull;
			Cull.Init(
				NaniteView,
				LocalBoundsCenter,
				LocalBoundsExtent,
				InstanceData.NonUniformScale,
				DynamicData.LocalToTranslatedWorld,
				DynamicData.PrevLocalToTranslatedWorld );

			Cull.bSkipWPODisableDistance |= DrawCommandDesc.bMaterialAlwaysEvaluatesWorldPositionOffset;

			Cull.Distance( PrimitiveData );
			// NOTE: Cull.Distance may set Cull.bEnableWPO
			bEnableWPO = bEnableWPO && Cull.bEnableWPO;
			
			// NOTE: Doing this for just the primary view (not mip views) is fine for current logic
			const bool bAllowWPO = VirtualShadowMapIsWPOAllowed(PrimitiveData, NaniteView.TargetLayerIndex);
			bEnableWPO = bEnableWPO && bAllowWPO;
			bool bCacheAsStatic = ShouldCacheInstanceAsStatic(InstanceId, (NaniteView.Flags & NANITE_VIEW_FLAG_UNCACHED), bAllowWPO, NaniteView.SceneRendererPrimaryViewId);

			// TODO: Set Cull.bIsStaticGeometry and dynamic depth range culling?

#if USE_HZB_OCCLUSION
			bool bUseStaticOcclusion = bCacheAsStatic;
#endif
			Cull.SetupCacheDependentState(bCacheAsStatic, InstanceData.InstanceId, NaniteView.SceneRendererPrimaryViewId);

			if (!bEnableWPO)
			{
				// Disable the Evaluate WPO culling flag if WPO was disabled
				CullingFlags &= ~INSTANCE_CULLING_FLAG_EVALUATE_WPO;
			}

			Cull.ScreenSize(DrawCommandDesc.MinScreenSize, DrawCommandDesc.MaxScreenSize);

			Cull.GlobalClipPlane();

			bool bInvalidatePages = ShouldMaterialInvalidateShadowCache(PrimitiveData, bEnableWPO)
					|| GetInstanceViewData(InstanceId, NaniteView.SceneRendererPrimaryViewId).bIsDeforming;
			
			FFrustumCullData FrustumCull = (FFrustumCullData)0;
			BRANCH
			if( Cull.bIsVisible )
			{
				// Note: Cull.PageFlagMask may be set in Cull.Frustum()
				FrustumCull = Cull.Frustum();
			}

			StatsBufferInterlockedAdd(VSM_STAT_NON_NANITE_INSTANCES_TOTAL, NaniteView.TargetNumMipLevels);

			BRANCH
			if (Cull.bIsVisible)
			{
				// Compute estimated footprint in the VSM base level - note that we don't use the actual footprint because it changes under rotation and also with edge clipping which makes it unstable when used for culling.
				float PixelEstRadius = CalcClipSpaceRadiusEstimate(Cull.bIsOrtho, InstanceData, Cull.LocalToTranslatedWorld, NaniteView.ViewToClip) * float(VSM_VIRTUAL_MAX_RESOLUTION_XY);

				// Loop over mip levels and count number of output visible instances
				for (uint MipLevel = 0U; MipLevel < uint(NaniteView.TargetNumMipLevels); ++MipLevel)
				{
					uint MipViewId = MipLevel * TotalPrimaryViews + PrimaryViewId;
					FNaniteView MipView = GetNaniteView(MipViewId);
					uint VirtualShadowMapId = uint(MipView.TargetLayerIndex);
					FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(MipView.TargetLayerIndex);

					FScreenRect Rect = GetScreenRect(MipView.ViewRect, FrustumCull, 4);

					bool bDetailGeometry = IsDetailGeometry(bCacheAsStatic, false, PixelEstRadius);
					// Update for next clip level
					PixelEstRadius *= 0.5f;

					Rect = VirtualShadowMapGetUncachedScreenRect( Rect, VirtualShadowMapHandle, MipLevel );
					uint4 RectPages = VirtualShadowMapGetPageRect( Rect );

					if (OverlapsAnyValidPage(VirtualShadowMapHandle, MipLevel, Rect, Cull.PageFlagMask, bDetailGeometry, Cull.bUseReceiverMask))
					{
#if VSM_GENERATE_STATS
						TotalPageArea += float(GetInclusiveRectArea(RectPages));
						NumVisibleLevels += 1.0f;
#endif
#if USE_HZB_OCCLUSION
						if (Cull.bViewHZB)
						{
							// We are using the current-frame HZB and can use any page whether it is cached or not
							// TODO: Figure out which flag(s) we actually want here. Also need to check the non-nanite one?
							if (!IsVisibleMaskedHZB(VirtualShadowMapHandle, MipLevel, Rect, true, true, 0U, 0xFFFFFFFF, bUseStaticOcclusion))
							{
								StatsBufferInterlockedInc(VSM_STAT_NON_NANITE_INSTANCES_HZB_CULLED);
								continue;
							}
						}
#endif // USE_HZB_OCCLUSION

						uint NumMappedPages = 0U;
						{
							const uint MarkPageDirtyFlags = VirtualShadowMapGetMarkPageDirtyFlags(bCacheAsStatic, bInvalidatePages, bAllowWPO, false /*bInvalidateLODDelta*/);

							uint2 RectPagesSize = (RectPages.zw + 1u) - RectPages.xy;
							bool bIsSmallJob = RectPagesSize.x * RectPagesSize.y <= MAX_SINGLE_THREAD_MARKING_AREA;

							// NOTE: Large job only necessary if we actually have flags to write
							bool bDoLargeJob = !bIsSmallJob && (MarkPageDirtyFlags != 0);

							uint LargeJobIndex = 0U;
							if (bDoLargeJob)
							{
								// allocate slot to defer the work and use all threads in group
								InterlockedAdd(NumSharedMarkingJobs, 1U, LargeJobIndex);
								bMarkingJobQueueOverflow = bMarkingJobQueueOverflow || LargeJobIndex >= MARKING_JOB_QUEUE_SIZE;
							}

							if (bIsSmallJob || bMarkingJobQueueOverflow)
							{
								FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(VirtualShadowMapHandle, MipLevel);

								for (uint Y = RectPages.y; Y <= RectPages.w; ++Y)
								{
									for (uint X = RectPages.x; X <= RectPages.z; ++X)
									{
										if (MarkPageDirty(PageTableLevelOffset, uint2(X, Y), MipLevel, Cull.PageFlagMask, MarkPageDirtyFlags))
										{
											// Count pages if we're touching all of them anyway
											++NumMappedPages;
										}
									}
								}
							}
							else
							{
								if (bDoLargeJob)
								{
									// defer the work and use all threads in group
									FSharedMarkingJob Job;
									Job.RectPages = RectPages;
									Job.VirtualShadowMapId = VirtualShadowMapId;
									Job.MipLevel = MipLevel;
									Job.FlagMask = Cull.PageFlagMask;
									Job.MarkPageDirtyFlags = MarkPageDirtyFlags;
									SharedMarkingJobs[LargeJobIndex] = PackJob(Job);
								}
								// Must assume we have mapped pages (or defer the cmd write also...)
								NumMappedPages = 1U;
							}
						}

						if (NumMappedPages > 0U)
						{
							++ThreadTotalForAllViews;
							bVisibleInstancesOverflow |= !WriteCmd(MipViewId, InstanceId, Payload.IndirectArgIndex, CullingFlags, bCacheAsStatic);
						}
					}
					else
					{
						StatsBufferInterlockedInc(VSM_STAT_NON_NANITE_INSTANCES_PAGE_MASK_CULLED);
					}
				}
			}
			else
			{
				StatsBufferInterlockedAdd(VSM_STAT_NON_NANITE_INSTANCES_FRUSTUM_CULLED, NaniteView.TargetNumMipLevels);
			}
		}
		
		StatsBufferInterlockedAdd(VSM_STAT_NON_NANITE_INSTANCES_DRAWN, ThreadTotalForAllViews);
		StatsBufferInterlockedEnableFlags(VSM_STAT_OVERFLOW_FLAGS, VSM_STAT_OVERFLOW_FLAG_MARKING_JOB_QUEUE, bMarkingJobQueueOverflow, true);
		StatsBufferInterlockedEnableFlags(VSM_STAT_OVERFLOW_FLAGS, VSM_STAT_OVERFLOW_FLAG_VISIBLE_INSTANCES, bVisibleInstancesOverflow, true);

		// Accumulate total number of instances for each indirect argument, is also used to allocate space and output compact range of instances later
		InterlockedAdd(DrawIndirectArgsBufferOut[Payload.IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1], ThreadTotalForAllViews);

#if VSM_GENERATE_STATS
		if (NumPageAreaDiagnosticSlots > 0U && TotalPageArea > LargeInstancePageAreaThreshold)
		{
			for (uint Index = 0U; Index < NumPageAreaDiagnosticSlots; ++Index)
			{
				uint PrevArea = 0U;
				// Store these after the stats slots
				InterlockedMax(OutStatsBuffer[VSM_STAT_NUM + 2U * Index], TotalPageArea, PrevArea);
				// If the area was greater then we store the persistent Primitive ID after the area.
				// This obviously has a data race, so numbers & IDs will only be approximately correct, could use 64-bit atomics instead...
				if (PrevArea < TotalPageArea)
				{
					OutStatsBuffer[VSM_STAT_NUM + 2U * Index + 1U] = PrimitiveData.PersistentPrimitiveIndex;
					break;
				}
			}
		}
#endif
	}
	GroupMemoryBarrierWithGroupSync();
	uint NumMarkingJobs = NumSharedMarkingJobs;
	for (uint JobIndex = 0U; JobIndex < min(MARKING_JOB_QUEUE_SIZE, NumMarkingJobs); ++JobIndex)
	{
		FSharedMarkingJob Job = UnpackJob(SharedMarkingJobs[JobIndex]);
		FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(Job.VirtualShadowMapId);
		FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(VirtualShadowMapHandle, Job.MipLevel);
		uint2 RectPagesSize = (Job.RectPages.zw + 1u) - Job.RectPages.xy;
		for (uint Index = GroupThreadIndex; Index < RectPagesSize.x * RectPagesSize.y; Index += NUM_THREADS_PER_GROUP)
		{
			uint LocalPageY = uint(floor((float(Index) + 0.5f) * rcp(RectPagesSize.x)));
			uint2 vPage = Job.RectPages.xy + uint2(Index - RectPagesSize.x * LocalPageY, LocalPageY);

			// We don't need to add the correct flag mask here, as we don't look at the VSM_FLAG_DETAIL_GEOMETRY for invalidaiton processing
			MarkPageDirty(PageTableLevelOffset, vPage, Job.MipLevel, Job.FlagMask, Job.MarkPageDirtyFlags);
		}
	}
}

Buffer<uint> DrawIndirectArgsBuffer;
RWBuffer<uint> InstanceIdOffsetBufferOut;
RWStructuredBuffer<uint> OutputOffsetBufferOut;
RWStructuredBuffer<uint> TmpInstanceIdOffsetBufferOut;
uint NumIndirectArgs;

/**
 * Separate pass to allocate space, needs to run once the final space requirements are known. We buffer the page/instance-draw info and reshuffle later.
 * TODO: Possibly just re-run the culling process in the output pass, saves storing stuff, but may cost more and runs the risk of the passes disagreeing e.g., due to rounding or whatever.
 */
[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void AllocateCommandInstanceOutputSpaceCs(uint IndirectArgIndex : SV_DispatchThreadID)
{
	if (IndirectArgIndex < NumIndirectArgs)
	{
		uint CommandInstanceCount = DrawIndirectArgsBuffer[IndirectArgIndex * INDIRECT_ARGS_NUM_WORDS + 1];
		uint CommandInstanceOffset = 0U;
		if (CommandInstanceCount > 0U)
		{
			InterlockedAdd(OutputOffsetBufferOut[0], CommandInstanceCount, CommandInstanceOffset);
		}
		InstanceIdOffsetBufferOut[IndirectArgIndex] = CommandInstanceOffset;
		// Store second copy for use during output pass (as we need the first offset buffer during the actual rendering)
		TmpInstanceIdOffsetBufferOut[IndirectArgIndex] = CommandInstanceOffset;
	}

	// Also set up indirect dispatch args for the output pass (OutputCommandInstanceLists)
	//if (IndirectArgIndex == 0)
	//{
	//	uint NumVisibleInstances = VisibleInstanceCountBuffer[0];
	//	// ...dispatch args to process all the visible instances
	//}
}

StructuredBuffer<FVSMVisibleInstanceCmd> VisibleInstances;
StructuredBuffer <uint> VisibleInstanceCountBuffer;
//RWStructuredBuffer<uint> TmpInstanceIdOffsetBufferOut;
RWStructuredBuffer<uint> InstanceIdsBufferOut;
RWStructuredBuffer<uint> PageInfoBufferOut;


[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void OutputCommandInstanceListsCs(uint VisibleInstanceIndex : SV_DispatchThreadID)
{
	uint NumVisibleInstances = VisibleInstanceCountBuffer[0];

	if (VisibleInstanceIndex < NumVisibleInstances)
	{
		FVSMVisibleInstanceCmd VisibleInstanceCmd = VisibleInstances[VisibleInstanceIndex];
		const uint InstanceId = VisibleInstanceCmd.InstanceIdAndFlags & INSTANCE_ID_MASK;
		const uint CullingFlags = VisibleInstanceCmd.InstanceIdAndFlags >> INSTANCE_ID_NUM_BITS;

		// Scatter the instance ID & other data.
		uint InstanceIdOutputOffset = 0;
		InterlockedAdd(TmpInstanceIdOffsetBufferOut[VisibleInstanceCmd.IndirectArgIndex], 1U, InstanceIdOutputOffset);
		// TODO: maybe repack as uint2 since that might be better for these type of presumably scalar loads.
		InstanceIdsBufferOut[InstanceIdOutputOffset] = PackInstanceCullingOutput(InstanceId, 0u, CullingFlags);
		PageInfoBufferOut[InstanceIdOutputOffset] = VisibleInstanceCmd.PackedPageInfo;
	}
}

