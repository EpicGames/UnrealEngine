// Copyright Epic Games, Inc. All Rights Reserved.

// Do not use shared samplers as it requires the View uniform buffer, which is not bound for this shader.
#define USE_HZB_SHARED_SAMPLERS 0

#include "../Common.ush"
#include "../SceneData.ush"
#include "../ViewData.ush"
#include "VirtualShadowMapPageCacheCommon.ush"
#include "VirtualShadowMapCacheInvalidation.ush"
#include "../WaveOpUtil.ush"


// Per-view isolate persistent buffer, bound for each dispatch
RWStructuredBuffer<uint> InOutViewInstanceState;
RWBuffer<uint> OutInvalidationArgs;
RWStructuredBuffer<uint> OutInvalidationQueue;

uint PackInvalidationItem(uint InstanceId, int SceneRendererPrimaryViewId, bool bStateChange)
{
	check(InstanceId < (1u << 24u));
	check(SceneRendererPrimaryViewId >= 0 && SceneRendererPrimaryViewId < (1u << 7u));
	return (InstanceId << 8u) | (uint(SceneRendererPrimaryViewId) << 1u) | select(bStateChange, 1u, 0u);
}

struct FInvalidationItem
{
	uint InstanceId;
	int SceneRendererPrimaryViewId;
	bool bStateChange;
};

FInvalidationItem UnpackInvalidationItem(uint PackedItem)
{
	FInvalidationItem Result;
	Result.InstanceId = PackedItem >> 8u;
	Result.SceneRendererPrimaryViewId = (PackedItem >> 1u) & 0x7fu; // 7 bits
	Result.bStateChange = (PackedItem & 1u) != 0u;
	return Result;
}

// Buffer containing one bit for each scene renderer view, such that we can random access in the renderer. Stores a copy of the output state but is transient
RWStructuredBuffer<uint> OutCacheInstanceAsDynamic;

int SceneRendererViewId;
int StateWordStride;
int MaxValidInstanceIndex;
int InvalidationQueueMax;


#define FOREACH_SET_BIT_BEGIN(MaskWord, BitIndexVarName, BitMaskVarName) \
	{	\
		uint MaskWordTemp = (MaskWord);	\
		while (MaskWordTemp != 0u)	\
		{	\
			uint BitIndexVarName = firstbitlow(MaskWordTemp);	\
			uint BitMaskVarName = 1u << BitIndexVarName;	\
			MaskWordTemp ^= BitMaskVarName;


#define FOREACH_SET_BIT_END()	} }

#ifdef VSMUpdateViewInstanceStateCS

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void VSMUpdateViewInstanceStateCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	// Process 32 instances per thread by loading a whole word of bits & performing bitwise operations. Scalar equivalent logic as comments.
	uint WordIndex = DispatchThreadId.x;
	if (WordIndex >= StateWordStride)
	{
		return;
	}

	// This is equivalent to a range test, returning if the instance index is out of range. Instead we create a mask to ensure we don't set any bits past the end of the currently used range (as they need to be cleared in the future).
	const uint NumInstancesOutOfRange = uint(max(0, (int(WordIndex + 1u) * 32) - MaxValidInstanceIndex));
	const uint AllInstancesMask = 0xFFFFFFFFu >> NumInstancesOutOfRange;

	// bool bIsDeforming = IsInstanceDeforming(InstanceIndex);
	uint IsDeformingMask = LoadInstanceDeformingWord(WordIndex, SceneRendererViewId);
	// First word contains the static/dynamic flag
	// bool bIsCachedAsDynamic = LoadBit(InstanceIndex, InOutViewInstanceState);
	uint CachedAsDynamicMask = InOutViewInstanceState[WordIndex];
	// Second word contains the new/old flag
	// bool bIsTracked = LoadBit(InstanceIndex + StateWordStride, InOutViewInstanceState);
	uint TrackedMask = InOutViewInstanceState[WordIndex + StateWordStride];

	// Update CachedAsDynamic flags in the persistent buffer: state is now whatever it was marked as.
	InOutViewInstanceState[WordIndex] = IsDeformingMask;

	// Set the instances to tracked in the persistent buffer (skip the write if this didn't change).
	if (AllInstancesMask != TrackedMask)
	{
		InOutViewInstanceState[WordIndex + StateWordStride] = AllInstancesMask;
	}


	// IF the instance is new - i.e., the TrackedMask bit is cleared - then it must already have been marked for invalidation (added). 
	// We let that process deal with that.	
	// TODO: The on-added invalidations should be moved to happen _after_ this so it can look at the correct state bits.
	//         For example, instance is added and should be dynamic, in this case the on-add invalidations will clear the static page but not the dynamic (since we have not figured the per-view state out yet)
	//         This is not _very_ bad, since static will invalidate dynamic also & therefore there won't be any visual artifacts, but there will be some cost if we add a lot of instances (which is not super-likely).

	// TODO: potential hysterisis?? Needs K-bit counter for each instance.

	// Detect instances that changed one way or the other - typically this will produce zero set bits, which is what makes this kernel efficient
	// Going from untracked is not a state change, since there was no previous state. This avoids newly added things from triggering invalidations.
	// XOR to get any bit that disagrees between new state & tracked state, and mask off any untracked
	// bool bStateChanged = (bIsDeforming != bIsCachedAsDynamic) && bIsTracked;
	uint StateChangeMask = (IsDeformingMask ^ CachedAsDynamicMask) & TrackedMask;

	// Next we write out the transient state for the current scene renderer, in theory we could alias the data from the persistent state for this bit but this is complex because it is stored in a separate buffer per view
	// and for Nanite & VSM we want them to be able to random access.

	// 2x as we have 2 bits per view, one for the current state and one for whether there was a transition (maybe we want to interleave these?.
	int BaseStateOffset = (2 * SceneRendererViewId) * StateWordStride + WordIndex;
	// Unconditionally write out the current state to the transient buffer for scene renderer views.
	OutCacheInstanceAsDynamic[BaseStateOffset] = IsDeformingMask;

	// Store the transitioned state after the current state flags, this is used to determine whether to render on top
	OutCacheInstanceAsDynamic[BaseStateOffset + StateWordStride] = StateChangeMask;

	// Only invalidate on transitions TO dynamic.
	// This leaves static pages with double shadow but this is hopefully ok (better than costly invalidations). It should be possible to extend the logic here to make decision based on primitive info.
	// bool bShouldInvalidate = bStateChanged && bIsDeforming;
	uint InvalidationMask = StateChangeMask & IsDeformingMask;

	// Finally output the invalidation work. This may look costly, but happens only if there are any transitions, which will not be the common case.
	// Use loop over bits to keep memory access pattern ok, trading some atomics, it would be possible to reorganize to only do one atomic/group but TBD (unsure how to abstract such a construction).
	FOREACH_SET_BIT_BEGIN(InvalidationMask, BitIndex, BitMask)
	{
		uint WriteOffset = 0;
		WaveInterlockedAddScalarInGroups(OutInvalidationArgs[3], OutInvalidationArgs[0], THREAD_GROUP_SIZE, 1, WriteOffset);
		if (WriteOffset < InvalidationQueueMax)
		{
			// Output instance ID and view index packed into one , we could output the mask instead and do this loop in the invalidation kernel but this will provide better load balancing.
			OutInvalidationQueue[WriteOffset] = PackInvalidationItem(WordIndex * 32u + BitIndex, SceneRendererViewId, (BitMask & StateChangeMask) != 0u);
		}
	}
	FOREACH_SET_BIT_END()
}

#endif // VSMUpdateViewInstanceStateCS

#ifdef ProcessInvalidationQueueGPUCS

Buffer<uint> InvalidationArgs;
StructuredBuffer<uint> InvalidationQueue;

[numthreads(THREAD_GROUP_SIZE, 1, 1)]
void ProcessInvalidationQueueGPUCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex, uint3 DispatchThreadId : SV_DispatchThreadID)
{
	uint NumInstances = InvalidationArgs[3];
	uint InvalidationQueueIndex = DispatchThreadId.x;
	if (InvalidationQueueIndex >= min(NumInstances, uint(InvalidationQueueMax)))
	{
		return;
	}

	FInvalidationItem Item = UnpackInvalidationItem(InvalidationQueue[InvalidationQueueIndex]);
	FInstanceSceneData InstanceSceneData = GetInstanceSceneData(Item.InstanceId);

	if (!IsValidInstanceAndCastShadows(InstanceSceneData))
	{
		return;
	}

	// Brute force all the lights...
	for (uint VirtualShadowMapIndex = 0; VirtualShadowMapIndex < VirtualShadowMap.NumFullShadowMaps; ++VirtualShadowMapIndex)
	{
		const FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromFullIndex(VirtualShadowMapIndex);
		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

		// Skip shadow maps for which this scene renderer primary view was not used.

		// 1. If there is no persistent ID then this must be a transient view & we can use the current ID (this should also disable fetching history data etc)
		int SceneRendererPrimaryViewId = ProjectionData.SceneRendererPrimaryViewId;
		if (ProjectionData.PersistentViewId != -1)
		{
			SceneRendererPrimaryViewId = RemapPersisitentViewId(ProjectionData.PersistentViewId);
		}

		// Id the view doesn't match that of the item, skip
		if (SceneRendererPrimaryViewId != Item.SceneRendererPrimaryViewId)
		{
			continue;
		}

		// Always just invalidate the dynamic pages since we want to keep the static on transitions.
		uint OverrideInvalidationFlags = VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC;
		InvalidateInstancePages(ProjectionData, InstanceSceneData, OverrideInvalidationFlags);
	}
}

#endif // ProcessInvalidationQueueGPUCS

#ifdef VSMResetInstanceStateCS 
#include "../InstanceCulling/InstanceCullingLoadBalancer.ush"

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void VSMResetInstanceStateCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	FInstanceWorkSetup WorkSetup = InstanceCullingLoadBalancer_Setup(GroupId, GroupThreadIndex, 0U);
	if (!WorkSetup.bValid)
	{
		return;
	}
	const uint WordIndex = WorkSetup.Item.InstanceDataOffset + uint(WorkSetup.LocalItemIndex);
	const uint PrimitiveId = WorkSetup.Item.Payload;
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(PrimitiveId);

	uint FirstInstanceIdInWord = WordIndex * 32u;
	uint LastInstanceIdInWord = FirstInstanceIdInWord + 32u;
	uint LastInstanceInPrimitive = PrimitiveData.InstanceSceneDataOffset + PrimitiveData.NumInstanceSceneDataEntries;
	// 2 cases:
	// 1. we are writing a whole word, so just write it
	if (PrimitiveData.InstanceSceneDataOffset <= FirstInstanceIdInWord && LastInstanceIdInWord <= LastInstanceInPrimitive)
	{
		// clear cache as dynamic
		InOutViewInstanceState[WordIndex] = 0u;
		// and more importantly "tracked" bit
		InOutViewInstanceState[WordIndex + StateWordStride] = 0u;
	}
	else // 2. we are writing a sub-range: create a mask and use atomics
	{
		uint FirstBit = uint(max(0, int(PrimitiveData.InstanceSceneDataOffset) - int(FirstInstanceIdInWord)));
		uint FirstMask = 0xFFFFFFFFu << FirstBit;
		uint LastBit = uint(max(0, int(LastInstanceIdInWord) - int(LastInstanceInPrimitive)));
		uint LastMask = 0xFFFFFFFFu >> LastBit;
		uint Mask = FirstMask & LastMask;
		InterlockedAnd(InOutViewInstanceState[WordIndex], Mask);
		InterlockedAnd(InOutViewInstanceState[WordIndex + StateWordStride], Mask);
	}
}

#endif
