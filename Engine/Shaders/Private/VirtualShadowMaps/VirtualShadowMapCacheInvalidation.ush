// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapCacheInvalidation.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../SceneData.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapPageOverlap.ush"
#include "VirtualShadowMapPageCacheCommon.ush"
#include "../ScreenSpaceDenoise/SSDDefinitions.ush"		// For LIGHT_TYPE's
#include "../Nanite/NaniteHZBCull.ush"

RWStructuredBuffer<FPhysicalPageMetaData> PhysicalPageMetaDataOut;

// We write the invalidation flags to the page request flags buffer to be consumed by the next
// physical page update pass.
RWTexture2D<uint> OutPageRequestFlags;

bool IsValidInstanceAndCastShadows(FInstanceSceneData InstanceSceneData)
{
	if (!InstanceSceneData.ValidInstance)
	{
		return false;
	}

	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceSceneData.PrimitiveId);
	const bool bCastShadows = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS) != 0U;
	const bool bForceHidden = (PrimitiveData.VisibilityFlags & PRIMITIVE_VISIBILITY_FLAG_FORCE_HIDDEN) != 0u;

	// TODO: test the flag on the instance instead once it is updated correctly InstanceSceneData.CastShadows
	return bCastShadows && !bForceHidden;
}

bool IsValidInstanceAndCastShadows(uint InstanceId)
{
	FInstanceSceneData InstanceSceneData = GetInstanceSceneData(InstanceId);
	return IsValidInstanceAndCastShadows(InstanceSceneData);
}

void InvalidateInstancePages(FVirtualShadowMapProjectionShaderData ProjectionData, FInstanceSceneData InstanceSceneData, uint OverrideInvalidationFlags)
{
	const bool bIsNanite = InstanceSceneData.NaniteRuntimeResourceID != 0xFFFFFFFFu;

	const bool bDirectionalLight = (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL);

	// NOTE: This is the *shadow view*'s translated world, not primary view
	float4x4 LocalToTranslatedWorld = DFMultiplyAffineTranslationDemote(InstanceSceneData.LocalToWorld, ProjectionData.PreViewTranslation);

	// distance cull invalidations for local lights
	if (!bDirectionalLight)
	{
		float InstanceRadius = length(InstanceSceneData.LocalBoundsExtent * InstanceSceneData.NonUniformScale.xyz);
		float3 TranslatedWorldCenter = mul(float4(InstanceSceneData.LocalBoundsCenter, 1.0f), LocalToTranslatedWorld).xyz;
		if (length2(TranslatedWorldCenter) > Square(ProjectionData.LightRadius + InstanceRadius))
		{
			return;
		}
	}

	// Go back to clip space
	float4x4 UVToClip;
	UVToClip[0] = float4(2, 0, 0, 0);
	UVToClip[1] = float4(0, -2, 0, 0);
	UVToClip[2] = float4(0, 0, 1, 0);
	UVToClip[3] = float4(-1, 1, 0, 1);

	const bool bIsOrtho = bDirectionalLight;
	const bool bNearClip = !bDirectionalLight;

	FFrustumCullData Cull = BoxCullFrustum(
		InstanceSceneData.LocalBoundsCenter,
		InstanceSceneData.LocalBoundsExtent,
		LocalToTranslatedWorld,
		mul(ProjectionData.TranslatedWorldToShadowUVMatrix, UVToClip),
		ProjectionData.ShadowViewToClipMatrix, bIsOrtho, bNearClip, false);

	float PixelEstRadius = CalcClipSpaceRadiusEstimate(bIsOrtho, InstanceSceneData, LocalToTranslatedWorld, ProjectionData.ShadowViewToClipMatrix) * float(VSM_VIRTUAL_MAX_RESOLUTION_XY);

	if (Cull.bIsVisible)
	{
		bool bAllowWPO = VirtualShadowMapIsWPOAllowed(GetPrimitiveData(InstanceSceneData.PrimitiveId), ProjectionData.VirtualShadowMapHandle);
		bool bShouldCacheAsStatic = ShouldCacheInstanceAsStatic(InstanceSceneData.InstanceId, ProjectionData.bUnCached, bAllowWPO, ProjectionData.SceneRendererPrimaryViewId);
		uint InvalidationFlags = (bShouldCacheAsStatic ? VSM_EXTENDED_FLAG_INVALIDATE_STATIC : VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC);
		if (OverrideInvalidationFlags != 0u)
		{
			InvalidationFlags = OverrideInvalidationFlags;
		}
		bool bInvalidateStaticPage = (InvalidationFlags & VSM_EXTENDED_FLAG_INVALIDATE_STATIC) != 0;
		
		if (ProjectionData.VirtualShadowMapHandle.IsSinglePage())
		{
			// Invalidate the one page and we're done
			FVSMPageOffset PageOffset = CalcPageOffset(ProjectionData.VirtualShadowMapHandle, VSM_MAX_MIP_LEVELS - 1, uint2(0, 0));
			InterlockedOr(OutPageRequestFlags[PageOffset.GetResourceAddress()], VSM_EXTENDED_FLAG_INVALIDATE_STATIC);
			return;
		}

		// 2. figure out overlap and all that
		// case #1 mip-map VSM - loop all mip levels, case #2 clipmap, just one 'mip level'
		int NumMipLevels = (ProjectionData.ClipmapLevelCountRemaining <= 0) ? VSM_MAX_MIP_LEVELS : 1;
		{
			for (int MipLevel = ProjectionData.MinMipLevel; MipLevel < NumMipLevels; ++MipLevel)
			{
				int ViewDim = int(uint(VSM_VIRTUAL_MAX_RESOLUTION_XY) >> MipLevel);
				FScreenRect Rect = GetScreenRect(int4(0, 0, ViewDim, ViewDim), Cull, 4);

				// Add a small epsilon to the HZB depth test
				// This is to handle the rare case where an object that is fully parallel to the 
				// light's near plane might self-occlude the HZB test due to minor precision differences
				// in the computation. While rare, this can come up with things like point lights and
				// axis aligned boxes.
				Rect.Depth += 1e-8f;

				// NOTE: We need to look at all allocated pages here, not just the uncached ones that are used for rendering
				// We need to include even ones that are not referenced this frame
				uint4 RectPages = VirtualShadowMapGetAllocatedPageRect(Rect, ProjectionData.VirtualShadowMapHandle, MipLevel);

				bool bDetailGeometry = IsDetailGeometry(bInvalidateStaticPage, bIsNanite, PixelEstRadius);
				PixelEstRadius *= 0.5f;

				// Use Hierarchical mip test to speed up (allows skipping invalidating areas that don't have any flags anyway)
				if (OverlapsAnyValidPage(ProjectionData.VirtualShadowMapHandle, MipLevel, RectPages, VSM_FLAG_ALLOCATED, bDetailGeometry))
				{
#if USE_HZB_OCCLUSION
					FPageTestScreenRect HZBTestRect = SetupPageHZBRect(Rect, ProjectionData.VirtualShadowMapHandle, MipLevel);
#endif // USE_HZB_OCCLUSION

					// 3. do invalidation
					FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(ProjectionData.VirtualShadowMapHandle, MipLevel);
					for (uint y = RectPages.y; y <= RectPages.w; y++)
					{
						for (uint x = RectPages.x; x <= RectPages.z; x++)
						{
							FVSMPageOffset PageFlagOffset = CalcPageOffset(PageTableLevelOffset, MipLevel, uint2(x, y));
							uint PageFlags = VirtualShadowMapGetPageFlag(PageFlagOffset);

							uint TestFlags = VSM_FLAG_ALLOCATED 
								// Only invalidate dynamic pages marked with VSM_FLAG_DETAIL_GEOMETRY if bForceCacheDynamicCoarse is enabled for the SM
								// Note that this is also suppressed downstream so not super important to skip here perhaps.
								| ((ProjectionData.bForceCacheDynamicCoarse && !bInvalidateStaticPage) ? VSM_FLAG_DETAIL_GEOMETRY : 0u);
							if ((PageFlags & TestFlags) == TestFlags)
							{
#if USE_HZB_OCCLUSION
								if (!IsPageVisibleHZB(uint2(x, y), PageFlagOffset, false, HZBTestRect, true))
								{
									continue;
								}
#endif // USE_HZB_OCCLUSION
								// We don't look up the page table here because pages that weren't requested this frame
								// are not in the page table, but still need to be invalidated. Thus we mark a bit in
								// the page request flags instead which will get picked up during the next page update pass.
								InterlockedOr(OutPageRequestFlags[PageFlagOffset.GetResourceAddress()], InvalidationFlags);
							}
						}
					}
				}
			}
		}
	}
}

void InvalidateInstancePages(FVirtualShadowMapHandle VirtualShadowMapHandle, uint InstanceId, uint OverrideInvalidationFlags)
{
	FInstanceSceneData InstanceSceneData = GetInstanceSceneData(InstanceId);
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	InvalidateInstancePages(ProjectionData, InstanceSceneData, OverrideInvalidationFlags);
}
