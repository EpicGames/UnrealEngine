// Copyright Epic Games, Inc. All Rights Reserved.

// Do not use shared samplers as it requires the View uniform buffer, which is not bound for this shader.
#define USE_HZB_SHARED_SAMPLERS 0

#include "../Common.ush"
#include "../SceneData.ush"
#include "VirtualShadowMapPageCacheCommon.ush"
#include "VirtualShadowMapCacheInvalidation.ush"
#include "../WaveOpUtil.ush"
#include "../InstanceCulling/InstanceCullingLoadBalancer.ush"

struct FInstanceInvalidationPayload
{
	int VirtualShadowMapId;
	uint Flags;
};

FInstanceInvalidationPayload DecodeInstanceInvalidationPayload(uint Payload)
{
	FInstanceInvalidationPayload Result;
	Result.VirtualShadowMapId = Payload >> VSM_INVALIDATION_PAYLOAD_FLAG_BITS;
	Result.Flags = Payload & ((1U<<VSM_INVALIDATION_PAYLOAD_FLAG_BITS)-1U);
	return Result;
}

[numthreads(CS_1D_GROUP_SIZE_X, 1, 1)]
void InvalidateInstancePagesLoadBalancerCS(
	uint3 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex,
	uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FInstanceWorkSetup WorkSetup = InstanceCullingLoadBalancer_Setup(GroupId, GroupThreadIndex, 0U, true);
	if (!WorkSetup.bValid)
	{
		return;
	}
	const uint InstanceId = WorkSetup.Item.InstanceDataOffset + uint(WorkSetup.LocalItemIndex);

	if (!IsValidInstanceAndCastShadows(InstanceId))
	{
		return;
	}

	FInstanceInvalidationPayload Payload = DecodeInstanceInvalidationPayload(WorkSetup.Item.Payload);
	bool bForceStatic = (Payload.Flags & VSM_INVALIDATION_PAYLOAD_FLAG_FORCE_STATIC) != 0;

	InvalidateInstancePages(FVirtualShadowMapHandle::MakeFromId(Payload.VirtualShadowMapId), InstanceId, bForceStatic ? VSM_EXTENDED_FLAG_INVALIDATE_STATIC : 0u);
}
