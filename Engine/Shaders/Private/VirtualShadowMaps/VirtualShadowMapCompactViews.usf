// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapCompactViews.usf: 
=============================================================================*/

#include "../Common.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "/Engine/Shared/SceneCullingDefinitions.h"

#define NANITE_MULTI_VIEW 1
#include "../Nanite/NaniteDataDecode.ush"

RWStructuredBuffer< FPackedNaniteView > CompactedViewsOut;
RWStructuredBuffer< uint > CompactedViewsAllocationOut;

RWStructuredBuffer< FViewDrawGroup > InOutViewDrawRanges;
uint NumViewRanges;

// Cube map (6 faces) with max mips is the current limiter
// In theory clipmaps could go to higher counts but they are practically limited to 32 currently
// due to the coarse page bitmask and a few other things
#define VSM_MAX_VIEWS_PER_GROUP (VSM_MAX_MIP_LEVELS*6)

groupshared uint OutputViewId[VSM_MAX_VIEWS_PER_GROUP];
groupshared uint OutputViewCount;
groupshared uint OutputViewRangeOffset;

uint PackViewIdMip(uint PrimaryViewId, uint MipLevel)
{
	return (PrimaryViewId << 4) | MipLevel;
}

uint2 UnpackViewIdMip(uint Packed)
{
	return uint2(Packed >> 4, Packed & 0xF);
}

[numthreads(VSM_MAX_VIEWS_PER_GROUP, 1, 1)]
void CompactViewsVSM_CS(
	uint GroupId : SV_GroupID,
	uint ThreadId : SV_GroupIndex)
{
	// Order-preserving compaction
	// Could be a wave ops, but portable wave size assumptions are inconvenient here
	// Fast enough to just do this scalar as long a we vectorize the actual copy below
	if (ThreadId == 0)
	{
		const uint ViewGroupIndex = GroupId.x;
		uint OutputViewOffset = 0U;

		FViewDrawGroup ViewDrawGroup = InOutViewDrawRanges[ViewGroupIndex];
		for (uint PrimaryViewId = ViewDrawGroup.FirstView; PrimaryViewId < ViewDrawGroup.FirstView + ViewDrawGroup.NumViews; ++PrimaryViewId)
		{
			FNaniteView PrimaryNaniteView = GetNaniteView(PrimaryViewId);

			for (uint MipLevel = 0; MipLevel < (uint)PrimaryNaniteView.TargetNumMipLevels; MipLevel++)
			{
				uint4 RectPages = VirtualShadowMap.UncachedPageRectBounds[PrimaryNaniteView.TargetLayerIndex * VSM_MAX_MIP_LEVELS + MipLevel];
				if (all(RectPages.zw >= RectPages.xy))
				{
					OutputViewId[OutputViewOffset] = PackViewIdMip(PrimaryViewId, MipLevel);
					OutputViewOffset += 1U;
				}
			}
		}

		OutputViewCount = OutputViewOffset;

		InterlockedAdd(CompactedViewsAllocationOut[1], OutputViewCount, OutputViewRangeOffset);
		{
			FViewDrawGroup ViewDrawGroupResult;
			ViewDrawGroupResult.FirstView = OutputViewRangeOffset;
			ViewDrawGroupResult.NumViews = OutputViewCount;
			InOutViewDrawRanges[ViewGroupIndex] = ViewDrawGroupResult;
		}
	}

	GroupMemoryBarrierWithGroupSync();

	// Copy/generate any views that survived into their compacted locations
	if (ThreadId < OutputViewCount)
	{
		uint CompactedOutputOffset = OutputViewRangeOffset + ThreadId;

		// Load associated primary view
		uint2 PrimaryViewIdMipLevel = UnpackViewIdMip(OutputViewId[ThreadId]);
		uint PrimaryViewId = PrimaryViewIdMipLevel.x;
		uint MipLevel = PrimaryViewIdMipLevel.y;
		FPackedNaniteView PrimaryNaniteView = InViews[PrimaryViewId];

		// Modify a few fields to make it the appropriate mip view
		FPackedNaniteView MipView = PrimaryNaniteView;
		MipView.TargetLayerIdX_AndMipLevelY_AndNumMipLevelsZ.y = MipLevel;

		uint MipDim = VSM_VIRTUAL_MAX_RESOLUTION_XY >> MipLevel;
		MipView.ViewRect = uint4(0, 0, MipDim, MipDim);
		MipView.HZBTestViewRect = MipView.ViewRect;
		MipView.ViewSizeAndInvSize = float4(MipDim, MipDim, 1.0f / MipDim, 1.0f / MipDim);

		float ScaleFactor = float(1U << MipLevel);
		float RcpScaleFactor = 1.0f / ScaleFactor;
		MipView.LODScales = RcpScaleFactor * PrimaryNaniteView.LODScales;
		MipView.ClipSpaceScaleOffset.x =  PrimaryNaniteView.ClipSpaceScaleOffset.x * RcpScaleFactor;
		MipView.ClipSpaceScaleOffset.y =  PrimaryNaniteView.ClipSpaceScaleOffset.y * RcpScaleFactor;
		MipView.ClipSpaceScaleOffset.z =  MipView.ClipSpaceScaleOffset.x - 1.0f;
		MipView.ClipSpaceScaleOffset.w = -MipView.ClipSpaceScaleOffset.y + 1.0f;

		MipView.SVPositionToTranslatedWorld[0] *= ScaleFactor;
		MipView.SVPositionToTranslatedWorld[1] *= ScaleFactor;

		// Write final mip view
		CompactedViewsOut[CompactedOutputOffset] = MipView;
	}
}
