// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ComputeShaderUtils.ush"
#include "/Engine/Shared/SceneCullingDefinitions.h"

uint NumWorkGroups;
StructuredBuffer< FViewDrawGroup > InViewDrawRanges;
RWStructuredBuffer< FInstanceCullingGroupWork > InOutInstanceCullingWorkGroups;

[numthreads(64, 1, 1)]
void ComputeExplicitChunkDrawsViewMask(
	uint3 GroupId : SV_GroupID,
	uint GroupThreadIndex : SV_GroupIndex)
{
	const uint DispatchThreadId = GetUnWrappedDispatchThreadId(GroupId, GroupThreadIndex, 64);
	if (DispatchThreadId < NumWorkGroups)
	{
		FInstanceCullingGroupWork WorkGroup = InOutInstanceCullingWorkGroups[DispatchThreadId];
		const FViewDrawGroup ViewDrawGroup = InViewDrawRanges[WorkGroup.ViewGroupId];
		
		WorkGroup.ActiveViewMask = (1u << ViewDrawGroup.NumViews) - 1u;
		
		InOutInstanceCullingWorkGroups[DispatchThreadId] = WorkGroup;
	}
}
