// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "VirtualShadowMapStats.ush"
#include "/Engine/Shared/NaniteDefinitions.h"

StructuredBuffer<uint>	InStatsBuffer;
RWBuffer<uint>			AccumulatedStatsBufferOut;

// Pull out a bunch of nanite stats for our CSV as well
StructuredBuffer<FNaniteStats> NaniteStatsBuffer;

[numthreads(1, 1, 1)]
void CopyStatsCS()
{
	uint Index;
	InterlockedAdd(AccumulatedStatsBufferOut[0], 1, Index);
	if (Index < MAX_STAT_FRAMES)
	{
		uint Offset = 1 + Index * VSM_STAT_NUM;
		for (uint StatInd = 0; StatInd < VSM_STAT_NUM; ++StatInd)
		{
			AccumulatedStatsBufferOut[Offset + StatInd] = InStatsBuffer[StatInd];
		}

		// Grab a few nanite stats
		FNaniteStats NaniteStats = NaniteStatsBuffer[0];
		AccumulatedStatsBufferOut[Offset + VSM_STAT_NANITE_TRIANGLES] = NaniteStats.NumTris;
		AccumulatedStatsBufferOut[Offset + VSM_STAT_NANITE_INSTANCES_MAIN] = NaniteStats.NumMainInstancesPostCull;
		AccumulatedStatsBufferOut[Offset + VSM_STAT_NANITE_INSTANCES_POST] = NaniteStats.NumPostInstancesPostCull;
	}
}
