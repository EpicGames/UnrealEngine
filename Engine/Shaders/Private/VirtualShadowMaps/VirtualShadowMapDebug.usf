// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPageManagement.usf: 
=============================================================================*/

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "VirtualShadowMapPageOverlap.ush"
#include "VirtualShadowMapPageCacheCommon.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "../Visualization.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"

uint DebugTargetWidth;
uint DebugTargetHeight;
uint BorderWidth;
uint VisualizeModeId;	// VIRTUAL_SHADOW_MAP_VISUALIZE_*
int VirtualShadowMapId;

StructuredBuffer<FPhysicalPageMetaData> PhysicalPageMetaData;
RWTexture2D<float4> OutVisualize;

static const float3 BlackColor = float3(0.0f, 0.0f, 0.0f);


float3 DebugVisualizeClipmap(float2 PixelUV, FVirtualShadowMapProjectionShaderData ProjectionData0)
{
	float UVScale = 0.0f;
	int LastValidLevelIndex = -1;

#if 1
	// First figure out a scale factor to zoom in on non-empty pages.
	for (int Index = ProjectionData0.ClipmapLevelCountRemaining - 1; Index >= 0; --Index)
	{
		FVirtualShadowMapHandle LevelSmHandle = ProjectionData0.VirtualShadowMapHandle.MakeOffset(Index);
		uint4 AllocatedBounds = VirtualShadowMap.AllocatedPageRectBounds[LevelSmHandle.GetDataIndex() * VSM_MAX_MIP_LEVELS];

		if (all(AllocatedBounds.xy <= AllocatedBounds.zw))
		{
			FVirtualShadowMapProjectionShaderData Level = GetVirtualShadowMapProjectionData(LevelSmHandle);

			float3 OriginTranslatedWorld = Level.ClipmapWorldOriginOffset;
			float4 ShadowUVz = mul(float4(OriginTranslatedWorld, 1.0f), Level.TranslatedWorldToShadowUVMatrix);
			const float2 PageCenter = ShadowUVz.xy * float(VSM_LEVEL0_DIM_PAGES_XY);
			float2 AllocatedExtentPages = max(abs(float2(AllocatedBounds.xy) - PageCenter), abs(float2(AllocatedBounds.zw) - PageCenter));
			
			float NewUVScale = (2.0f + max(AllocatedExtentPages.x, AllocatedExtentPages.y)) / float(VSM_LEVEL0_DIM_PAGES_XY / 2);
		
			if (LastValidLevelIndex < 0)
			{
				UVScale = NewUVScale;
				LastValidLevelIndex = Index;
			}
			else
			{
				UVScale = max(UVScale, NewUVScale / float(1u << (LastValidLevelIndex - Index)));
			}
		}
	}
#endif

	if (LastValidLevelIndex < 0)
	{
		return BlackColor;
	}

	const float2 CenteredPixelUV = PixelUV - 0.5f;

	int PageOverlayCount = 0;

	float3 ResultColor = BlackColor;
	for (int Index = LastValidLevelIndex; Index >= 0; --Index)
	{
		FVirtualShadowMapHandle LevelSmHandle = ProjectionData0.VirtualShadowMapHandle.MakeOffset(Index);

		FVirtualShadowMapProjectionShaderData Level = GetVirtualShadowMapProjectionData(LevelSmHandle);

		float3 OriginTranslatedWorld = Level.ClipmapWorldOriginOffset;
		float4 ShadowUVz = mul(float4(OriginTranslatedWorld, 1.0f), Level.TranslatedWorldToShadowUVMatrix);

		int2 vPage = int2((CenteredPixelUV * UVScale + ShadowUVz.xy) * float(VSM_LEVEL0_DIM_PAGES_XY));
		UVScale *= 2.0f;

		if (any(vPage < 0) || any(vPage >= VSM_LEVEL0_DIM_PAGES_XY))
		{
			continue;
		}

		FVSMPageOffset PageOffset = CalcPageOffset(LevelSmHandle, 0, vPage);
		FShadowPhysicalPage pPage = ShadowGetPhysicalPage(PageOffset);

		if (pPage.bThisLODValid)
		{
			FPhysicalPageMetaData MetaData = PhysicalPageMetaData[VSMPhysicalPageAddressToIndex(pPage.PhysicalAddress)];

			PageOverlayCount += 1;
			ResultColor = lerp(IntToColor(vPage.x + (vPage.y << 10U)), IntToColor(Level.ClipmapLevel), 0.75f);

		}
	}

	if (PageOverlayCount > 0)
	{
		//ResultColor = GreenToRedTurbo(float(PageOverlayCount - 1) / float(8));
	}

	return ResultColor;
}

float4 DebugVisualizeClipmap(uint2 OutputPixel, int VirtualShadowMapId)
{
	// Figure out the square area do draw in (could be any sub-rect)
	uint SquareDomainDim = min(DebugTargetWidth, DebugTargetHeight) - BorderWidth * 2U;

	// fill background pixels
	if (any(or(OutputPixel < BorderWidth, OutputPixel >(SquareDomainDim + BorderWidth))))
	{
		return float4(0, 0, 0, 0);
	}

	// The pixel we are drawing in 0-1 range UV within the domain
	float2 PixelUV = float2(OutputPixel - BorderWidth) / float(SquareDomainDim);

	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId));
	if (ProjectionData.LightType != LIGHT_TYPE_DIRECTIONAL)
	{
		return float4(0, 0, 0, 0);
	}

	return float4(DebugVisualizeClipmap(PixelUV, ProjectionData), 1.0);
}


[numthreads(VSM_DEFAULT_CS_GROUP_XY, VSM_DEFAULT_CS_GROUP_XY, 1)]
void DebugVisualizeVirtualSmCS(uint2 PixelPos : SV_DispatchThreadID)
{
	if (any(PixelPos >= uint2(DebugTargetWidth, DebugTargetHeight)))
	{
		return;
	}
	// NOTE: Important to *not* write output if it isn't a recognized mode, as a different
	// pass may write that output instead.
	float4 Output = float4(1, 0, 1, 0);

	if (VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_VIRTUAL_SPACE)
	{
		Output = DebugVisualizeClipmap(PixelPos, VirtualShadowMapId);
	}

	if (Output.a > 0)
	{
		OutVisualize[PixelPos] = Output;
	}
}

int VisualizeVirtualShadowMapId;
float VisualizeNaniteOverdrawScale;
Texture2D DebugTexture;
SamplerState DebugTextureSampler;

float4 TonemapProjectionDebugTexturePS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
) : SV_Target0
{
	bool bOneLightVisible = VisualizeVirtualShadowMapId >= 0;

	float2 UV = UVAndScreenPos.xy;
	float4 Output = Texture2DSample(DebugTexture, DebugTextureSampler, UV);

	if (VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SMRT_RAY_COUNT)
	{
		if (Output.a > 0)
		{
			float TotalRayCount = Output.r;
			float TotalMaxRayCount = Output.g;
			float IntensityMultiplier = bOneLightVisible ? 1.0 : smoothstep(0, 64, TotalMaxRayCount)+0.5;
			Output.rgb = GreenToRedTurbo(IntensityMultiplier * TotalRayCount / TotalMaxRayCount);
		}
	}
	else if (VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW)
	{
		const float OverdrawScale = clamp(VisualizeNaniteOverdrawScale / 100.0f, 0.0f, 1.0f);
		const float OverdrawCount = Output.r;
		const float OverdrawColor = 1 - exp2(-OverdrawCount * OverdrawScale);
		Output = float4(ColorMapInferno(OverdrawColor), 0.0);
		Output.a = smoothstep(-0.1, 0.1, dot(Output.rgb, 1.0/3.0));
	}
	return Output;
}
