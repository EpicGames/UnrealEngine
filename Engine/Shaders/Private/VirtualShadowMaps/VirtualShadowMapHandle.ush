// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "../Common.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"

/**
 * Abstraction to facilitate passing meta-data (which is sometimes compiletime constant) along in addition to just an index.
 */
struct FVirtualShadowMapHandle
{
	uint Id;
	bool bIsSinglePageSM;

	static FVirtualShadowMapHandle MakeFromId(uint VirtualShadowMapId)
	{
		FVirtualShadowMapHandle Result;
		Result.Id = VirtualShadowMapId;
		Result.bIsSinglePageSM = VirtualShadowMapId < uint(VSM_MAX_SINGLE_PAGE_SHADOW_MAPS);
		return Result;
	}

	static FVirtualShadowMapHandle MakeFromIdDirectional(uint VirtualShadowMapId)
	{
		FVirtualShadowMapHandle Result;
		Result.Id = VirtualShadowMapId;
		Result.bIsSinglePageSM = false;
		return Result;
	}

	static FVirtualShadowMapHandle MakeFromFullIndex(uint FullVirtualShadowMapIndex)
	{
		FVirtualShadowMapHandle Result;
		Result.Id = FullVirtualShadowMapIndex + VSM_MAX_SINGLE_PAGE_SHADOW_MAPS;
		Result.bIsSinglePageSM = false;
		return Result;
	}

	static FVirtualShadowMapHandle MakeInvalid()
	{
		FVirtualShadowMapHandle Result;
		Result.Id = 0xFFFFFFFFu;
		Result.bIsSinglePageSM = false;
		return Result;
	}

	/**
	 * Note: signed offset, can go both ways.
	 */
	FVirtualShadowMapHandle MakeOffset(int Offset)
	{
		checkSlow(IsValid());
		checkSlow(int(Id) + Offset >= 0);

		FVirtualShadowMapHandle Result;
		Result.Id = uint(int(Id) + Offset);
		Result.bIsSinglePageSM = bIsSinglePageSM;
		return Result;
	}

	bool IsValid()
	{
		return Id != 0xFFFFFFFFu;
	}

	bool IsSinglePage()
	{
		return bIsSinglePageSM;
	}

	uint GetDataIndex()
	{
		return Id;
	}
};
