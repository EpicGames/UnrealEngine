// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapLightGrid.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../LightGridCommon.ush"

FCulledLightsGridHeader VirtualShadowMapGetLightsGridHeader(uint2 PixelPos, float SceneDepth)
{
	uint EyeIndex = View.StereoPassIndex;

	uint GridLinearIndex = min(VirtualShadowMap.MaxLightGridEntryIndex, ComputeLightGridCellIndex(PixelPos, SceneDepth, EyeIndex));
	FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridLinearIndex);

	// Replace light count with our pruned count
	checkStructuredBufferAccessSlow(VirtualShadowMap.NumCulledLightsGrid, GridLinearIndex);
	CulledLightGridHeader.NumLights = VirtualShadowMap.NumCulledLightsGrid[GridLinearIndex];
	return CulledLightGridHeader;
}

FLocalLightData VirtualShadowMapGetLocalLightData(FCulledLightsGridHeader GridHeader, uint Index)
{
	uint EyeIndex = View.StereoPassIndex;

	const uint ListIndex = GridHeader.DataStartIndex + Index;
	checkStructuredBufferAccessSlow(VirtualShadowMap.LightGridData, ListIndex);
	const uint LightGridLightIndex = VirtualShadowMap.LightGridData[ListIndex];
	return GetLocalLightData(LightGridLightIndex, EyeIndex);
}
