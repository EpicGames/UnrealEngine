// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapMaskBitsCommon.ush:
=============================================================================*/

#pragma once

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "../LightShaderParameters.ush"
#include "../LightGridCommon.ush"
#include "../VirtualShadowMaps/VirtualShadowMapLightGrid.ush"
#include "../VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"

float GetVirtualShadowMapMaskForLight(
	uint4 ShadowMask,
	uint2 PixelPos,
	float SceneDepth,
	int VirtualShadowMapId,
	float3 TranslatedWorldPosition,
	inout uint OutLightIndex)
{
	// Light grid is relative to the view
	const uint2 LocalPixelPos = PixelPos.xy - uint2(View.ViewRectMinAndSize.xy);
	const FCulledLightsGridHeader LightGridHeader = VirtualShadowMapGetLightsGridHeader(LocalPixelPos, SceneDepth);	

	// We can only handle so many lights in our output encoding right now, so no purpose in computing more
	uint LightCount = min(GetPackedShadowMaskMaxLightCount(), LightGridHeader.NumLights);

	OutLightIndex = 0;
	for (; OutLightIndex < LightCount; ++OutLightIndex)
	{
		const FLocalLightData LightData = VirtualShadowMapGetLocalLightData(LightGridHeader, OutLightIndex);
		if (LightData.Internal.VirtualShadowMapId == VirtualShadowMapId)
		{
			break;
		}
	}

	float LightAttenuation = 1.0f;
	if (OutLightIndex < LightCount)
	{
		// TODO: Dither!
		LightAttenuation = UnpackShadowMask(ShadowMask, OutLightIndex);
	}
	else
	{
		const float StaticBias = 200.0f;
		// Fall back to a single sample VSM lookup
		FVirtualShadowMapSampleResult VSMResult = SampleVirtualShadowMapLocal(
			VirtualShadowMapId,
			TranslatedWorldPosition,
			StaticBias
		);
		LightAttenuation = VSMResult.ShadowFactor;
	}

	return LightAttenuation;
}

float GetVirtualShadowMapMaskForLight(
	uint4 ShadowMask,
	uint2 PixelPos,
	float SceneDepth,
	int VirtualShadowMapId,
	float3 TranslatedWorldPosition)
{
	uint OutDummyLightIndex = 0;
	return GetVirtualShadowMapMaskForLight(ShadowMask, PixelPos, SceneDepth, VirtualShadowMapId, TranslatedWorldPosition, OutDummyLightIndex);
}