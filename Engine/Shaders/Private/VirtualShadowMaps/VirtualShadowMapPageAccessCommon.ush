// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPageAccessCommon.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../BitPacking.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "VirtualShadowMapHandle.ush"

// NOTE: These page flags are combined hierarchically using bitwise *OR*, so plan/negate them appropriately
// Marks pages that are allocated
#define VSM_FLAG_ALLOCATED						(1U << 0)
// Marks pages whose dynamic pages are uncached
#define VSM_FLAG_DYNAMIC_UNCACHED				(1U << 1)
// Marks pages whose static pages are uncached
#define VSM_FLAG_STATIC_UNCACHED				(1U << 2)
// Marks pages that are _not_ coarse (i.e., "normal" pages) that should include all geometry and conversely to mark geometry that is 
// "detail geometry" and which can skip rendering to coarse pages
#define VSM_FLAG_DETAIL_GEOMETRY				(1U << 3)
// Marks page requested by primary visiblitity (pixels / froxels)
#define VSM_FLAG_PRIMARY_REQUEST				(1u << 4)
// Marks page that is requested by secondary visiblitity - that is prefiltered distant because it is the prefiltered fallback to some primary requested page
#define VSM_FLAG_SECONDARY_REQUEST				(1u << 5)

// If the total bit count needs to change be sure it doesn't overlow the page flags for all Hmips
#define VSM_PAGE_FLAGS_BITS_PER_HMIP (6U)
#define VSM_PAGE_FLAGS_BITS_MASK     ((1U<<VSM_PAGE_FLAGS_BITS_PER_HMIP)-1U)

// Convenience
#define VSM_FLAG_ANY_UNCACHED				(VSM_FLAG_DYNAMIC_UNCACHED | VSM_FLAG_STATIC_UNCACHED)
#define VSM_FLAG_AUX_MASK					(VSM_FLAG_DETAIL_GEOMETRY | VSM_FLAG_PRIMARY_REQUEST | VSM_FLAG_SECONDARY_REQUEST)
// Testing (see TestPageFlagMask)
// Flags in this subset are tested non-exclusively (pass if any in the mask is set in the flags)
#define VSM_FLAG_TEST_ANY_MASK (VSM_FLAG_ANY_UNCACHED | VSM_FLAG_ALLOCATED)
// Flags in this subset are tested exclusively (pass if all in mask are set in flags)
#define VSM_FLAG_TEST_ALL_MASK (VSM_FLAG_DETAIL_GEOMETRY | VSM_FLAG_PRIMARY_REQUEST | VSM_FLAG_SECONDARY_REQUEST)

// Stored in the physical meta data flags and must be higher than the  VSM_*_FLAG
// These are *NOT* valid to be used with hierarchical flag lookups!

// Set whenever a dynamic page has been initialized; i.e. it is cleared, or has just a copy of the static data
#define VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED	(1U << 6)
// Set whenever a static page has been initialized; i.e. it is cleared with no data in it
#define VSM_EXTENDED_FLAG_STATIC_INITIALIZED	(1U << 7)
// Set whenever a static page has something new written into it
#define VSM_EXTENDED_FLAG_DYNAMIC_DIRTY			(1U << 8)
// Set whenever a static page has something new written into it
#define VSM_EXTENDED_FLAG_STATIC_DIRTY			(1U << 9)
// The next time this page is rendered, clear the dynamic portion and re-render
#define VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC    (1U << 10)
// The next time this page is rendered, clear the static portion and re-render
#define VSM_EXTENDED_FLAG_INVALIDATE_STATIC     (1U << 11)
// Propogated from the NaniteView for convenience; allows skipping separate static/dynamic rendering and associated merge
#define VSM_EXTENDED_FLAG_VIEW_UNCACHED			(1U << 12)
// Set on physical pages that are unreferenced in the current render pass/frame. We can keep these alive and cached if space permits but otherwise ignore them.
#define VSM_EXTENDED_FLAG_UNREFERENCED			(1U << 13)
// A bit messy, but this flag forces a page to be considered cached even if an UNCACHED flag is set.
// This is currently used so that we still consider these pages during rendering/instance cull (generally for WPO disable/enable purposes)
// but will not actually clear, draw, or merge them this frame.
#define VSM_EXTENDED_FLAG_FORCE_CACHED			(1U << 14)
// Pages marked as deferred invalidation are invalidated in UpdatePhysicalPages usin a per-frame budget (r.Shadow.Virtual.DeferredInvalidationBudget). 
// The flag is not cleared until processed.
#define VSM_EXTENDED_FLAG_DEFERRED_INVALIDATION	(1U << 15)

#define VSM_EXTENDED_FLAG_ANY_INITIALIZED	(VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED | VSM_EXTENDED_FLAG_STATIC_INITIALIZED)
#define VSM_EXTENDED_FLAG_ANY_DIRTY			(VSM_EXTENDED_FLAG_DYNAMIC_DIRTY | VSM_EXTENDED_FLAG_STATIC_DIRTY)
#define VSM_EXTENDED_FLAG_ANY_INVALIDATED	(VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC | VSM_EXTENDED_FLAG_INVALIDATE_STATIC)

uint CalcLog2LevelDimsPages(uint Level)
{
	return VSM_LOG2_LEVEL0_DIM_PAGES_XY - Level;	// log2( VSM_LEVEL0_DIM_PAGES_XY >> Level )
}

uint CalcLevelDimsPages(uint Level)
{
	return 1u << CalcLog2LevelDimsPages( Level );
}

uint CalcLevelDimsTexels(uint Level)
{
	return uint(VSM_VIRTUAL_MAX_RESOLUTION_XY) >> Level;
}

uint2 CalcLevelOffsets(uint MipLevel)
{
	uint2 Result = uint2(0u, 0u);
	if (MipLevel > 0u)
	{
		// Lay out the mip tail below in address space so we can wrap the X coord using POT
		Result.y += VSM_LEVEL0_DIM_PAGES_XY;

		uint MaxMask = (1u << (VSM_MAX_MIP_LEVELS - 1)) - 1u;
		uint StartBit = VSM_MAX_MIP_LEVELS - MipLevel;
		Result.x += MaxMask & (MaxMask << StartBit);
	}
	return Result;
};

struct FVirtualSMLevelOffset
{
	bool bIsSinglePageSM;
	uint2 LevelTexelOffset;

	uint GetPacked()
	{
		return (LevelTexelOffset.x << 16u) | LevelTexelOffset.y;
	}

	static FVirtualSMLevelOffset Unpack(uint PackedLevelOffset)
	{
		FVirtualSMLevelOffset Result;
		Result.LevelTexelOffset.x = PackedLevelOffset >> 16u;
		Result.LevelTexelOffset.y = PackedLevelOffset & 0xFFFFu;
		// We can derive the bIsSinglePageSM flag from the level offset since it will always be in the 0th page
		Result.bIsSinglePageSM = all(Result.LevelTexelOffset < VSM_PAGE_SIZE);
		return Result;
	}
};


/**
 * Compute the offset for a mip level page table given a shadow map ID and a level.
 */
FVirtualSMLevelOffset CalcPageTableLevelOffset(FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel)
{
	FVirtualSMLevelOffset Result;
	Result.bIsSinglePageSM = VirtualShadowMapHandle.IsSinglePage();
	if (Result.bIsSinglePageSM)
	{
		// Map into single page at 0,0
		Result.LevelTexelOffset.y = VirtualShadowMapHandle.Id >> VSM_LOG2_PAGE_SIZE;
		Result.LevelTexelOffset.x = VirtualShadowMapHandle.Id & VSM_PAGE_SIZE_MASK;
	}
	else
	{
		// Skip the first one as we reserve that for the single-page bozos
		uint FullId = uint(VirtualShadowMapHandle.Id - VSM_MAX_SINGLE_PAGE_SHADOW_MAPS) + 1u; 
		Result.LevelTexelOffset.y = (FullId >> VirtualShadowMap.PageTableRowShift) * VSM_PAGE_TABLE_TEX2D_SIZE_Y;
		Result.LevelTexelOffset.x = (FullId & VirtualShadowMap.PageTableRowMask) * VSM_PAGE_TABLE_TEX2D_SIZE_X;

		Result.LevelTexelOffset += CalcLevelOffsets(MipLevel);
	}
	return Result;
}

/**
 * Compute the offset for a mip level page table given a shadow map ID and a level.
 */
FVirtualSMLevelOffset CalcPageTableLevelOffset(int VirtualShadowMapId, uint MipLevel)
{
	return CalcPageTableLevelOffset(FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId), MipLevel);
}

uint2 CalcPageOffsetInFullLevel(uint Level, uint2 PageAddress)
{
	return PageAddress;
}

/**
 * Offset of an individual page in the global page table data structure, whatever it may be.
 */
struct FVSMPageOffset
{
	uint2 TexelAddress;

	// Returns the address formated for resource (texture / buffer) access.
	uint2 GetResourceAddress()
	{
		return TexelAddress;
	}

	uint GetPacked()
	{
		return (TexelAddress.x << 16u) | TexelAddress.y;
	}

	static FVSMPageOffset Unpack(uint Packed) 
	{ 
		FVSMPageOffset Result;
		Result.TexelAddress.x = Packed >> 16u;
		Result.TexelAddress.y = Packed & 0xFFFFu;
		return Result;
	}
};

/**
 * Compute the offset for page within a level page table given a level and PageAddress.
 */
FVSMPageOffset CalcPageOffset(FVirtualSMLevelOffset LevelOffset, uint Level, uint2 PageAddress)
{
	//checkSlow(LevelOffset.bIsSinglePageSM == (LevelOffset.LevelOffset < VSM_MAX_SINGLE_PAGE_SHADOW_MAPS));
	FVSMPageOffset Result;
	Result.TexelAddress = LevelOffset.LevelTexelOffset;

	if (!LevelOffset.bIsSinglePageSM)
	{
		Result.TexelAddress += CalcPageOffsetInFullLevel(Level, PageAddress);
	}

	return Result;
}

FVSMPageOffset CalcPageOffset(FVirtualShadowMapHandle VirtualShadowMapHandle, uint Level, uint2 PageAddress)
{
	FVirtualSMLevelOffset LevelOffset = CalcPageTableLevelOffset(VirtualShadowMapHandle, Level);
	return CalcPageOffset(LevelOffset, Level, PageAddress);
}

FVSMPageOffset CalcPageOffset(int VirtualShadowMapId, uint Level, uint2 PageAddress)
{
	return CalcPageOffset(FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId), Level, PageAddress);
}

bool IsVirtualShadowMapPageAddressValid(int2 PageAddress, uint Level)
{
	return (all(PageAddress >= 0) && all(PageAddress < CalcLevelDimsPages(Level)));
}

// Linearlize a physical page address to a linear offset
uint VSMPhysicalPageAddressToIndex(uint2 PhysicalPageAddress)
{
	return (PhysicalPageAddress.y << VirtualShadowMap.PhysicalPageRowShift) + PhysicalPageAddress.x;
}

uint2 VSMPhysicalIndexToPageAddress(uint PageIndex)
{
	uint2 PageAddress;
	PageAddress.x = PageIndex  & VirtualShadowMap.PhysicalPageRowMask;
	PageAddress.y = PageIndex >> VirtualShadowMap.PhysicalPageRowShift;
	return PageAddress;
}


// Current page table format:
// NOTE: Some redundancy in flags and encoding, but we have spare bits for now
//   [0:9] PageAddress.x
//   [10:19] PageAddress.y
//   [20:25] LODOffset
//   [26:28] (currently unused)
//   [29] bIsSecondaryRequest
//   [30] bThisLODValidForRendering
//   [31] bAnyLODValid
struct FShadowPhysicalPage
{
	uint2 PhysicalAddress;	// Physical page address X, Y
	uint LODOffset;			// 0 if page is mapped at this mip/clipmap level; 1 if mapped at next courser level, etc. [0..64)
	bool bAnyLODValid;		// Valid physical page mapped at some LOD level
	bool bIsSecondaryRequest;		// VSM_PHYSICAL_PAGE_SECONDARY_REQUEST_FLAG
	bool bThisLODValidForRendering;		// Valid physical page mapped for rendering into
	bool bThisLODValid;		// Valid page mapped at this specific level (equivalent to bAnyMipValid && LODOffset == 0)
};

#define VSM_PHYSICAL_PAGE_SECONDARY_REQUEST_FLAG	0x20000000
#define VSM_PHYSICAL_PAGE_VALID_FOR_RENDERING_FLAG	0x40000000
#define VSM_PHYSICAL_PAGE_ANY_MIP_VALID_FLAG		0x80000000
#define VSM_PHYSICAL_PAGE_INVALID					0x00000000

uint ShadowEncodePageTable(uint2 PhysicalAddress, bool bValidForRendering, bool bIsSecondaryRequest)
{
	return (PhysicalAddress.y << 10) 
		| (PhysicalAddress.x) 
		| (bValidForRendering ?
			(VSM_PHYSICAL_PAGE_ANY_MIP_VALID_FLAG | VSM_PHYSICAL_PAGE_VALID_FOR_RENDERING_FLAG) :
			(VSM_PHYSICAL_PAGE_ANY_MIP_VALID_FLAG)) 
		| (bIsSecondaryRequest ? VSM_PHYSICAL_PAGE_SECONDARY_REQUEST_FLAG : 0u);
}
uint ShadowEncodePageTable(uint2 PhysicalAddress, uint LODOffset)
{
	// These are hierarchical pointers to coarser pages that are mapped, so never valid for rendering directly
	// See PropagateMappedMips
	return VSM_PHYSICAL_PAGE_ANY_MIP_VALID_FLAG | (LODOffset << 20) | (PhysicalAddress.y << 10) | (PhysicalAddress.x);
}

FShadowPhysicalPage ShadowDecodePageTable(uint Value)
{
	FShadowPhysicalPage Result;
	Result.PhysicalAddress = uint2(Value & 0x3FFu, (Value >> 10u) & 0x3FFu);
	Result.LODOffset = (Value >> 20u) & 0x3Fu;
	Result.bAnyLODValid = (Value & VSM_PHYSICAL_PAGE_ANY_MIP_VALID_FLAG) != 0u;
	Result.bIsSecondaryRequest = (Value & VSM_PHYSICAL_PAGE_SECONDARY_REQUEST_FLAG) != 0u;
	Result.bThisLODValidForRendering = (Value & VSM_PHYSICAL_PAGE_VALID_FOR_RENDERING_FLAG) != 0u;
	Result.bThisLODValid = Result.bAnyLODValid && Result.LODOffset == 0u;
	return Result;
}

FShadowPhysicalPage ShadowGetPhysicalPage(FVSMPageOffset VSMPageOffset, Texture2D<uint> InPageTable)
{
	return ShadowDecodePageTable(InPageTable[VSMPageOffset.GetResourceAddress()]);
}

FShadowPhysicalPage ShadowGetPhysicalPage(FVSMPageOffset VSMPageOffset)
{
	return ShadowGetPhysicalPage(VSMPageOffset, VirtualShadowMap.PageTable);
}

uint VirtualShadowMapGetPageFlag(FVSMPageOffset VSMPageOffset)
{
	return VirtualShadowMap.PageFlags[VSMPageOffset.GetResourceAddress()];
}

// Returns true if the page is valid *for rendering*
FShadowPhysicalPage VirtualToPhysicalTexelBase(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
{
	uint VPageX = VirtualTexelAddress.x >> VSM_LOG2_PAGE_SIZE;
	uint VPageY = VirtualTexelAddress.y >> VSM_LOG2_PAGE_SIZE;

	FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(PageTableLevelOffset, Level, uint2(VPageX, VPageY)));
	PhysicalTexelAddress = PhysicalPageEntry.PhysicalAddress * VSM_PAGE_SIZE + (VirtualTexelAddress & VSM_PAGE_SIZE_MASK);
	return PhysicalPageEntry;
}

// Returns true if the page is valid *for rendering*
bool VirtualToPhysicalTexelForRendering(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
{
	return VirtualToPhysicalTexelBase(PageTableLevelOffset, Level, VirtualTexelAddress, PhysicalTexelAddress).bThisLODValidForRendering;
}

// Returns true if the page is valid at all
bool VirtualToPhysicalTexel(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
{
	return VirtualToPhysicalTexelBase(PageTableLevelOffset, Level, VirtualTexelAddress, PhysicalTexelAddress).bThisLODValid;
}

bool VirtualToPhysicalTexel(FVirtualShadowMapHandle VirtualShadowMapHandle, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
{
	return VirtualToPhysicalTexel(CalcPageTableLevelOffset(VirtualShadowMapHandle, Level), Level, VirtualTexelAddress, PhysicalTexelAddress);
}

bool VirtualToPhysicalPage(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, int2 VirtualPageAddress, inout uint2 PhysicalPageAddress)
{
	if (any(VirtualPageAddress < 0) || any(VirtualPageAddress >= (VSM_LEVEL0_DIM_PAGES_XY >> Level)))
	{
		return false;
	}

	FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(PageTableLevelOffset, Level, uint2(VirtualPageAddress)));
	if (PhysicalPageEntry.bThisLODValid)
	{
		PhysicalPageAddress = PhysicalPageEntry.PhysicalAddress;
	}
	return (PhysicalPageEntry.bThisLODValid);
}

struct FShadowPageTranslationResult
{
	bool bValid;
	uint LODOffset;
	uint2 VirtualTexelAddress;
	float2 VirtualTexelAddressFloat;
	uint2 PhysicalTexelAddress;
};

// Finds the best-resolution mapped page at the given UV
FShadowPageTranslationResult ShadowVirtualToPhysicalUV(FVirtualShadowMapHandle VirtualShadowMapHandle, float2 ShadowMapUV, uint MinMipLevel)
{
	uint2 vPage = uint2(ShadowMapUV * VSM_LEVEL0_DIM_PAGES_XY);
	FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(VirtualShadowMapHandle, MinMipLevel, vPage >> MinMipLevel));

	FShadowPageTranslationResult Result;
	Result.bValid = PhysicalPageEntry.bAnyLODValid;
	Result.LODOffset = VirtualShadowMapHandle.IsSinglePage() ? (VSM_MAX_MIP_LEVELS - 1U) : (PhysicalPageEntry.LODOffset + MinMipLevel);
	// TODO: Can optimize this slightly based on relative offset
	Result.VirtualTexelAddressFloat = ShadowMapUV * float(CalcLevelDimsTexels(Result.LODOffset));
	Result.VirtualTexelAddress = uint2(Result.VirtualTexelAddressFloat);
	Result.PhysicalTexelAddress = PhysicalPageEntry.PhysicalAddress * VSM_PAGE_SIZE + (Result.VirtualTexelAddress & VSM_PAGE_SIZE_MASK);

	return Result;
}

struct FPageInfo
{
	uint ViewId;
	bool bStaticPage;		// Write to static page vs dynamic page
};

uint PackPageInfo(FPageInfo PageInfo)
{
	// TODO: Line up the bit encoding here with the max view count from Nanite
	return
		PageInfo.ViewId |
		(PageInfo.bStaticPage ? (1U << 16) : 0U);
}

FPageInfo UnpackPageInfo(uint PackedData)
{
	FPageInfo PageInfo;
	PageInfo.ViewId			= PackedData & 0xFFFF;
	PageInfo.bStaticPage	= ((PackedData >> 16) & 0x1) != 0;
	return PageInfo;
}

uint GetVirtualShadowMapStaticArrayIndex()
{
	return VirtualShadowMap.StaticCachedArrayIndex;
}

uint GetVirtualShadowMapHZBArrayIndex(bool bUseStaticOcclusion)
{
	return select(bUseStaticOcclusion, VirtualShadowMap.StaticHZBArrayIndex, 0);
}

// Gather a 2x2 footprint from a page table texture.
uint4 GatherPageTable(Texture2D<uint> PageTableTexture, uint2 TexelCoord, uint HMipLevel, float2 UVScaleFactor)
{
	uint2 TexelCoordMip = TexelCoord >> HMipLevel;
#if COMPILER_SUPPORTS_GATHER_LOD_RED
	// Offset to 2x2 footprint center and scale to UV space
	float MipScale = float(1U << HMipLevel);
	float2 UV = float2(TexelCoordMip + uint2(1U, 1U)) * MipScale * UVScaleFactor;
	uint4 Result2x2 = PageTableTexture.GatherLODRed(VirtualShadowMap.PageTableSampler, UV, HMipLevel);
#else
	uint4 TexelRectMip = uint4(TexelCoordMip.xy, TexelCoordMip.xy + uint2(1U, 1U));
	uint4 Result2x2 = uint4(
		PageTableTexture.Load(uint3(TexelRectMip.xw, HMipLevel)),	// (-, +)
		PageTableTexture.Load(uint3(TexelRectMip.zw, HMipLevel)),	// (+, +)
		PageTableTexture.Load(uint3(TexelRectMip.zy, HMipLevel)),	// (+, -)
		PageTableTexture.Load(uint3(TexelRectMip.xy, HMipLevel))	// (-, -)
	);
#endif
	return Result2x2;
}

uint4 GatherPageTableRW(RWTexture2D<uint> PageTableTexture, uint2 TexelCoord)
{
	uint4 TexelRect = uint4(TexelCoord.xy, TexelCoord.xy + uint2(1U, 1U));
	uint4 Result2x2 = uint4(
		PageTableTexture[TexelRect.xw],	// (-, +)
		PageTableTexture[TexelRect.zw],	// (+, +)
		PageTableTexture[TexelRect.zy],	// (+, -)
		PageTableTexture[TexelRect.xy]	// (-, -)
	);
	return Result2x2;
}
