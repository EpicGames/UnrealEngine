// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapPageCacheCommon.ush:
=============================================================================*/
#pragma once

#include "../Nanite/NaniteDataDecode.ush"
#include "VirtualShadowMapProjectionStructs.ush"

bool ShouldMaterialInvalidateShadowCache(FPrimitiveSceneData PrimitiveData, bool bEnableWPO)
{
	// Determine if this instance will cause page invalidations.
	bool bInvalidate = bEnableWPO;
	// Ignore WPO disable distance for this if any of the materials have bAlwaysEvaluateWorldPositionOffset = true.
	bInvalidate |= ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_ALWAYS_EVALUATE_WPO_MATERIALS) != 0u);
	// Unless, of course, the primitive opts to disable shadow cache invalidations.
	bInvalidate &= ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_DISABLE_MATERIAL_INVALIDATIONS) == 0u);

	return bInvalidate;
}

// NOTE: This logic must be constant, or else we could miss invalidations if it swaps suddenly.
// Any changes to the underlying data (i.e. evaluate WPO flag) need to generate a GPUScene update and
// associated VSM instance invalidation.
bool VirtualShadowMapIsWPOAllowed(FPrimitiveSceneData PrimitiveData, FVirtualShadowMapHandle VirtualShadowMapHandle)
{
	// If always evaluate WPO is enabled, that takes priority
	if ((PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_ALWAYS_EVALUATE_WPO_MATERIALS) != 0u)
	{
		return true;
	}

	bool bEvaluateWPO = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET) != 0u;
	bool bWPODisableDistance = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE) != 0u;
	if (bEvaluateWPO && bWPODisableDistance && VirtualShadowMapHandle.IsValid())
	{
		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
		// NOTE: ProjectionData set to a safe value for this test even if this is not a clipmap
		return ProjectionData.ClipmapLevelWPODistanceDisabledThresholdSquared <= PrimitiveData.InstanceWPODisableDistanceSquared;		
	}

	return bEvaluateWPO;
}

bool VirtualShadowMapIsWPOAllowed(FPrimitiveSceneData PrimitiveData, int VirtualShadowMapId)
{
	return VirtualShadowMapIsWPOAllowed(PrimitiveData, FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId));
}

bool GetCachePrimitiveAsDynamic(uint PersistentPrimitiveIndex)
{
	// Dynamic primitives are not tracked, so we treat them as dynamic since they can't be cached by definition.
	if (PersistentPrimitiveIndex >= GetSceneData().MaxPersistentPrimitiveIndex)
	{
		return true;
	}

	uint PrimitiveIdWordOffset = PersistentPrimitiveIndex / 32U;
	uint PrimitiveIdWordMask = 1U << (PersistentPrimitiveIndex % 32U);
	return (VirtualShadowMap.CachePrimitiveAsDynamic[PrimitiveIdWordOffset] & PrimitiveIdWordMask) != 0;
}

bool GetCacheInstanceAsDynamic(uint InstanceId, int SceneRendererPrimaryViewId)
{
	uint WordOffset = InstanceId / 32U;
	uint WordMask = 1U << (InstanceId % 32U);
#if USE_GLOBAL_GPU_SCENE_DATA
	// TODO: Need to implement this properly for previous frame data UE-368517
	return false;
#else
	uint ViewWordOffset = Scene.VSMCache.InstanceStateViewWordStride * SceneRendererPrimaryViewId * 2;
	return (Scene.VSMCache.CacheInstanceAsDynamic[ViewWordOffset + WordOffset] & WordMask) != 0;
#endif
}

/**
 * Returns true if, for the currect view & frame, the instance transitioned from being static to dynamic or vice versa.
 */
bool GetCacheTransitioned(uint InstanceId, int SceneRendererPrimaryViewId)
{
	uint WordOffset = InstanceId / 32U;
	uint WordMask = 1U << (InstanceId % 32U);
#if USE_GLOBAL_GPU_SCENE_DATA
	// TODO: Need to implement this properly for previous frame data UE-368517
	return false;
#else
	// Note: transitioned bits are stored after the other ones.
	uint ViewWordOffset = Scene.VSMCache.InstanceStateViewWordStride * SceneRendererPrimaryViewId * 2 + Scene.VSMCache.InstanceStateViewWordStride;
	return (Scene.VSMCache.CacheInstanceAsDynamic[ViewWordOffset + WordOffset] & WordMask) != 0;
#endif
}

bool ShouldCacheInstanceAsStatic(uint InstanceId, bool bViewUncached, bool bAllowWPO, int SceneRendererPrimaryViewId)
{
	if (bViewUncached)
	{
		return false;
	}
 
	FInstanceSceneData InstanceData = GetInstanceSceneDataUnchecked(InstanceId);
	FPrimitiveSceneData PrimitiveData = GetPrimitiveData(InstanceData.PrimitiveId);
 
	if (ShouldMaterialInvalidateShadowCache(PrimitiveData, bAllowWPO))
	{
		return false;
	}
	
	// TODO: Move this also to the per-instance bit vector?
	// Whole primitive decision takes precentence
	if (GetCachePrimitiveAsDynamic(PrimitiveData.PersistentPrimitiveIndex))
	{
		return false;
	}

	if (GetCacheInstanceAsDynamic(InstanceId, SceneRendererPrimaryViewId))
	{
		return false;
	}

	return true;
}
