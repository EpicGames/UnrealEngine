// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPageManagement.usf: 
=============================================================================*/

#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "VirtualShadowMapProjectionStructs.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapStats.ush"
#include "VirtualShadowMapPerPageDispatch.ush"

StructuredBuffer<FPhysicalPageMetaData> PhysicalPageMetaData;
RWTexture2D<uint> OutPageTable;
RWStructuredBuffer<uint4> OutUncachedPageRectBounds;
RWStructuredBuffer<uint4> OutAllocatedPageRectBounds;

// Mip a 2x1 (i.e., 8x4 bit) mask into a 4x4 mask
uint MipMask8x4(uint2 Mask2x1)
{
	// Do two 4x4 16-bit masks at once
	uint Packed = Mask2x1.x | (Mask2x1.y << 16u);

	// merge rows
	Packed |= Packed >> 4u;
	// merge columns
	Packed |= Packed >> 1u;
	// Mask off garbage
	Packed &= 0x5050505u;
	// Now we have mipped the bits but they are interleaved 
	// [0000 0H0G 0000 0F0E 0000 0D0C 0000 0B0A] - each letter represents a 2x2 input region
	// Move to [0000 00HG 0000 00FE 0000 00DC 0000 00BA]
	Packed |= Packed >> 1u;
	Packed &= 0x3030303u;
	// fold down the mipped rows (pairs of bits) to the correct place
	// [0000 00HG 00HG 00FE 0000 00DC 00DC 00BA]
	Packed |= Packed >> 4u;
	// interleave the two results
	// [0000 00HG 00HG 00FE 0000 00DC HGDC FEBA]
	Packed |= Packed >> 14u;
	// keep just the two last rows, mask off all the garbage
	// [0000 0000 0000 0000 0000 0000 HGDC FEBA]
	Packed &= 0xFFu;
	return Packed;
}

// Mip a 2x2 (i.e., 8x8 bit) mask into a 4x4 mask
uint MipMask8x8(uint4 Mask2x2)
{
	 // layout dictated by gather
	// x: (-, +)
	// y: (+, +)
	// z: (+, -)
	// w: (-, -)

	uint Result = (MipMask8x4(uint2(Mask2x2.x, Mask2x2.y)) << 8u)
		| MipMask8x4(uint2(Mask2x2.w, Mask2x2.z));

	return Result;
}

// Mip a 4x4 mask & use fractional position to position inside resulting 4x4 mask
uint MipMask4x4(uint Mask4x4, uint2 FracPos)
{
	// not 100% sure I need to mask off 0x33u
	uint Mask2x2 = MipMask8x4(uint2(Mask4x4, 0u)) & 0x33u;
	uint Shift = FracPos.y * 8u + FracPos.x * 2u;
	uint Res4x4 = Mask2x2 << Shift;
	return Res4x4;
}

#ifdef GenerateHierarchicalPageFlags

// Not yet set up in the UB
Texture2D<uint> InPageFlags;
Texture2D<uint> InPageReceiverMasks;

RWTexture2D<uint> OutPageFlagMips_0;
RWTexture2D<uint> OutPageFlagMips_1;
RWTexture2D<uint> OutPageFlagMips_2;
RWTexture2D<uint> OutPageFlagMips_3;
RWTexture2D<uint> OutPageFlagMips_4;
RWTexture2D<uint> OutPageFlagMips_5;
//RWTexture2D<uint> OutPageFlagMips_6;

RWTexture2D<uint> OutPageReceiverMaskMips_0;
RWTexture2D<uint> OutPageReceiverMaskMips_1;
RWTexture2D<uint> OutPageReceiverMaskMips_2;
RWTexture2D<uint> OutPageReceiverMaskMips_3;
RWTexture2D<uint> OutPageReceiverMaskMips_4;
RWTexture2D<uint> OutPageReceiverMaskMips_5;
RWTexture2D<uint> OutPageReceiverMaskMips_6;


bool ProcessMipLevel(RWTexture2D<uint> OutPageFlagMip, uint Flag, FVSMPageOffset PageOffset, uint HMipLevel)
{
	// shift to the appropriate mip level
	uint2 MipTexelAddress = PageOffset.TexelAddress >> HMipLevel;

	uint PreviousValue = 0;
	InterlockedOr(OutPageFlagMip[MipTexelAddress], Flag, PreviousValue);
	// If this was already the value for this HMip, then whoever did that will continue up the hierarhcy.
	return PreviousValue == Flag;
}


bool IntersectMask8x8(uint4 Mask2x2, uint2 Min, uint2 Max)
{
	// Assume: clamped to 8x8 rect

	// general idea: work on 8x8 and pull out relevant sub-ranges.

	// Make an 8-bit row mask & then select bits from that, easier than figuring out the ranges for each sub-range 
	uint NumXBits8 = Max.x - Min.x + 1u;
	uint XMask8 = BitFieldMaskU32(NumXBits8, Min.x);
	// XLow half (left half if you like)
	uint XMaskLow = XMask8 & 0xFu;
	uint XMaskHigh = XMask8 >> 4u;

	// make an 8-row mask
	uint MaxY4 = Max.y << 2u;
	uint MinY4 = Min.y << 2u;
	uint NumYBits = MaxY4 - MinY4 + 1u; // +1u instead of 4 to avoid overflow - we only use it to mask off 0x1111 anyway so no need to mask the last few
	uint YMask8 = BitFieldMaskU32(NumYBits, MinY4);
	uint RowMultLow  = 0x1111u & YMask8;
	uint RowMultHigh = 0x1111u & (YMask8 >> 16u);

	// x: (-, +) y: (+, +) z: (+, -) w: (-, -)

	uint4 TestMask;
	TestMask.w = XMaskLow * RowMultLow;
	TestMask.x = XMaskLow * RowMultHigh;
	TestMask.z = XMaskHigh * RowMultLow;
	TestMask.y = XMaskHigh * RowMultHigh;

	return (Mask2x2.x & TestMask.x) != 0u
		|| (Mask2x2.y & TestMask.y) != 0u
		|| (Mask2x2.z & TestMask.z) != 0u
		|| (Mask2x2.w & TestMask.w) != 0u;
}


void ProcessMaskMip4x4(RWTexture2D<uint> OutPageReceiverMaskMip, inout uint Mask4x4, inout uint2 MaskAddress)
{
	Mask4x4 = MipMask4x4(Mask4x4, MaskAddress & 1u);
	MaskAddress = MaskAddress >> 1u;
	uint PrevMask = 0u;
	InterlockedOr(OutPageReceiverMaskMip[MaskAddress], Mask4x4, PrevMask);
}

// Gather a 2x2 footprint given the upper-left coordinate
uint4 GatherReceiverMask(uint2 TexelCoord)
{
	return GatherPageTable(InPageReceiverMasks, TexelCoord, 0u, VirtualShadowMap.PageReceiverMaskTextureSizeInvSize.zw);
}

/**
* One thread per page physical table flag entry
*/
[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void GenerateHierarchicalPageFlags(uint ThreadId : SV_DispatchThreadID)
{
	// early out any overflowing threads.
	if (ThreadId >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}
	FPhysicalPageMetaData MetaData = PhysicalPageMetaData[ThreadId];

	if (MetaData.Flags == 0U)
	{
		return;
	}

	// Use the group ID to ensure the compiler knows it is scalar / uniform
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(MetaData.VirtualShadowMapId);
	FVSMPageOffset GlobalPageTableEntryIndex = CalcPageOffset(VirtualShadowMapHandle, MetaData.MipLevel, MetaData.PageAddress);

	uint Flag = InPageFlags[GlobalPageTableEntryIndex.GetResourceAddress()] & VSM_PAGE_FLAGS_BITS_MASK;
	if (Flag != 0u)
	{
		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

		uint MipLevel = MetaData.MipLevel;
		uint2 PageAddress = MetaData.PageAddress;
		// No hierarchy to propagate to for single-page pages.
		if (VirtualShadowMapHandle.IsSinglePage())
		{
			// Note: we need to set the page rect bounds for the last mip level, since that is the only one that is valid, logically, for a single-page VSM.
			//       This is important since this is what filters all the rendering that would otherwise try to draw stuff to the other levels.
			MipLevel = VSM_MAX_MIP_LEVELS - 1U;
		}

		// Compute the min/max rect of active pages
		uint PageBoundIndex = VirtualShadowMapHandle.GetDataIndex() * VSM_MAX_MIP_LEVELS + MipLevel;
		InterlockedMin(OutAllocatedPageRectBounds[PageBoundIndex].x, PageAddress.x);
		InterlockedMin(OutAllocatedPageRectBounds[PageBoundIndex].y, PageAddress.y);
		InterlockedMax(OutAllocatedPageRectBounds[PageBoundIndex].z, PageAddress.x);
		InterlockedMax(OutAllocatedPageRectBounds[PageBoundIndex].w, PageAddress.y);

		// Only add to the rendering page rect bounds if there is anything uncached
		if ((Flag & VSM_FLAG_ANY_UNCACHED) != 0)
		{
			InterlockedMin(OutUncachedPageRectBounds[PageBoundIndex].x, PageAddress.x);
			InterlockedMin(OutUncachedPageRectBounds[PageBoundIndex].y, PageAddress.y);
			InterlockedMax(OutUncachedPageRectBounds[PageBoundIndex].z, PageAddress.x);
			InterlockedMax(OutUncachedPageRectBounds[PageBoundIndex].w, PageAddress.y);
		}

		// No hierarchy to propagate to for single-page pages.
		if (VirtualShadowMapHandle.IsSinglePage())
		{
			return;
		}

		// Loop over H flag levels, this builds a mip pyramid over _each_ mip level in the page table
		// the 0-th level in this hiearchy is the page table mip level itself.
		uint MaxHLevel = VSM_MAX_MIP_LEVELS - MipLevel;

// Manually unrolled to work with "array" of textures, 
// NOTE: returns when done!
		bool bDone = false;
#define PROCESS_LEVEL(_Level_) \
		if (_Level_ < MaxHLevel && !bDone) { \
			bDone = ProcessMipLevel(OutPageFlagMips_##_Level_, Flag, GlobalPageTableEntryIndex, _Level_ + 1);	\
		}

		PROCESS_LEVEL(0);
		PROCESS_LEVEL(1);
		PROCESS_LEVEL(2);
		PROCESS_LEVEL(3);
		PROCESS_LEVEL(4);
		PROCESS_LEVEL(5);

#undef PROCESS_LEVEL

		BRANCH
		if (ProjectionData.bUseReceiverMask)
		{
			// 2x2 masks
			uint2 MaskAddress = GlobalPageTableEntryIndex.GetResourceAddress() * 2u;

			// 1. load 2x2 mask from level 0
			uint4 Mask8x8 = GatherReceiverMask(MaskAddress);
			//   construct 4x4 mask
			uint Mask4x4 = MipMask8x8(Mask8x8);
			//   Store back
			MaskAddress = MaskAddress >> 1u;
			uint PreviousMask4x4 = 0u;
			InterlockedOr(OutPageReceiverMaskMips_0[MaskAddress], Mask4x4, PreviousMask4x4);
			// early out if PreviousMask4x4 == Mask4x4? Will it ever?

			// 2. propagate upwards, at this point it is all self-similar as we shrink a 4x4 -> 2x2 word and store back
			ProcessMaskMip4x4(OutPageReceiverMaskMips_1, Mask4x4, MaskAddress);
			ProcessMaskMip4x4(OutPageReceiverMaskMips_2, Mask4x4, MaskAddress);
			ProcessMaskMip4x4(OutPageReceiverMaskMips_3, Mask4x4, MaskAddress);
			ProcessMaskMip4x4(OutPageReceiverMaskMips_4, Mask4x4, MaskAddress);
			ProcessMaskMip4x4(OutPageReceiverMaskMips_5, Mask4x4, MaskAddress);
			ProcessMaskMip4x4(OutPageReceiverMaskMips_6, Mask4x4, MaskAddress);			
		}
	}
}

#endif // GenerateHierarchicalPageFlags

#ifdef PropagateMappedMips


struct FMappedMipPropagator
{
	void Run(FPerPageDispatchSetup Setup)
	{
		if (Setup.VirtualShadowMapHandle.IsSinglePage())
		{
			return;
		}

		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(Setup.VirtualShadowMapHandle);
	
		if (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
		{
			const uint NumLevel0Entries = VSM_LEVEL0_DIM_PAGES_XY * VSM_LEVEL0_DIM_PAGES_XY;
			uint LoopEndXY = Setup.GetLoopEnd(0);
			for (uint PageY = Setup.LoopStart.y; PageY < LoopEndXY; PageY += Setup.LoopStride)
			{
				for (uint PageX = Setup.LoopStart.x; PageX < LoopEndXY; PageX += Setup.LoopStride)
				{
					uint2 Level0Page;
					Level0Page.x = PageX;
					Level0Page.y = PageY;
					// Directional lights propagate pages to their coarser/larger clipmap levels (and only use mip0 pages)
					// Each clipmap level is a separate VSM, so we gather any mapped coarser pages as necessary and write only our own page output
					// There's also technically a race similar to below with other threads writing the PT data we are reading,
					// but it's still deterministic as long as we only look at pages with "bThisLODValid".
					// There's some redundant work of course, but this shader is pretty cheap overall

					FVSMPageOffset Page0Offset = CalcPageOffset(Setup.VirtualShadowMapHandle, 0, Level0Page);
					FShadowPhysicalPage pPage0 = ShadowDecodePageTable(OutPageTable[Page0Offset.GetResourceAddress()]);

					BRANCH
					if (!pPage0.bThisLODValid)
					{
						const int OffsetScale = (VSM_LEVEL0_DIM_PAGES_XY >> 2);
						int2 BaseOffset = OffsetScale * ProjectionData.ClipmapCornerRelativeOffset;
						int2 BasePage   = int2(Level0Page) - BaseOffset;

						// Search for first mapped page past this one
						uint RemainingLevels = ProjectionData.ClipmapLevelCountRemaining;
						for (uint ClipmapOffset = 1; ClipmapOffset < RemainingLevels; ++ClipmapOffset)
						{
							FVirtualShadowMapHandle ClipmapLevelHandle = Setup.VirtualShadowMapHandle.MakeOffset(int(ClipmapOffset));
							FVirtualShadowMapProjectionShaderData LevelProjectionData = GetVirtualShadowMapProjectionData(ClipmapLevelHandle);								
							int2 LevelOffset = OffsetScale * LevelProjectionData.ClipmapCornerRelativeOffset;

							int2 LevelPage = (BasePage + (LevelOffset << ClipmapOffset)) >> ClipmapOffset;
							if (IsVirtualShadowMapPageAddressValid(LevelPage, 0))
							{
								FVSMPageOffset LevelPageOffset = CalcPageOffset(ClipmapLevelHandle, 0, uint2(LevelPage));
								FShadowPhysicalPage pPage = ShadowDecodePageTable(OutPageTable[LevelPageOffset.GetResourceAddress()]);
								if (pPage.bThisLODValid)
								{
									OutPageTable[Page0Offset.GetResourceAddress()] = ShadowEncodePageTable(pPage.PhysicalAddress, ClipmapOffset);
									break;
								}
							}
							else
							{
								// TODO: We're off the edge... can this ever even happen in practice given the construction?
							}
						}
					}
				}
			}
		}
		else
		{
			int MinLevel = Setup.MipLevelStart;
			uint LoopEndXY = Setup.GetLoopEnd(MinLevel);
			for (uint PageY = Setup.LoopStart.y; PageY < LoopEndXY; PageY += Setup.LoopStride)
			{
				for (uint PageX = Setup.LoopStart.x; PageX < LoopEndXY; PageX += Setup.LoopStride)
				{
					uint2 MinLevelPage;
					MinLevelPage.x = PageX;
					MinLevelPage.y = PageY;
			
					// Local lights propagate pages to their coarser mips
					int MappedPageLevel = -1;
					uint2 MappedPhysicalAddress = 0;

					for (int Level = (VSM_MAX_MIP_LEVELS - 1); Level >= MinLevel; --Level)
					{
						int LevelDelta = Level - MinLevel;
						uint2 vPage = MinLevelPage >> LevelDelta;
						FVSMPageOffset PageOffset = CalcPageOffset(Setup.VirtualShadowMapHandle, Level, vPage);
						FShadowPhysicalPage pPage = ShadowDecodePageTable(OutPageTable[PageOffset.GetResourceAddress()]);

						BRANCH
						if (pPage.bThisLODValid)
						{
							// This page is mapped, so leave it alone and propagate downwards
							MappedPageLevel = Level;
							MappedPhysicalAddress = pPage.PhysicalAddress;
						}
						else if( MappedPageLevel >= 0 )
						{
							// This page is not mapped; replace it with our suitably offset parent mapped page
							// Ensure only one thread writes each value to avoid races, but we read on all threads as the broadcast
							// Note that this can race with the other threads reading this value, but since bThisLODValid will
							// always be false on these updated pages the values will be ignored. As long as the writes to the page
							// table are atomic (currently a single DWORD), this is safe.
							if (all((vPage << LevelDelta) == MinLevelPage))
							{
								uint MipOffset = MappedPageLevel - Level;
								OutPageTable[PageOffset.GetResourceAddress()] = ShadowEncodePageTable(MappedPhysicalAddress, MipOffset);
							}
						}
					}
				}
			}
		}
	}
};

/**
* One thread per page in level 0, launched as 1d groups, with 2D grid with Y dim ==  NumFullShadowMaps.
* This is effectively just a big broadcast operation. There are more efficient ways to do this with
* fewer threads and wave ops, but given the page counts just relying on memory coalescing is
* good enough for now.
*/
[numthreads(PER_PAGE_THREAD_GROUP_SIZE_XY, PER_PAGE_THREAD_GROUP_SIZE_XY, 1)]
void PropagateMappedMips(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FPerPageDispatchSetup Setup; 
	FMappedMipPropagator Propagator;
	Setup.Execute(DispatchThreadId, Propagator);
}

#endif // PropagateMappedMips

#ifdef PropagateFilterableRequestsCS

RWTexture2D<uint> InOutPageRequestFlags;
RWTexture2D<uint> InOutPageReceiverMasks;

struct FRequestPropagator
{
	void Run(FPerPageDispatchSetup Setup)
	{
		if (Setup.VirtualShadowMapHandle.IsSinglePage())
		{
			return;
		}

		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(Setup.VirtualShadowMapHandle);
	
		if (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
		{
			const uint NumLevel0Entries = VSM_LEVEL0_DIM_PAGES_XY * VSM_LEVEL0_DIM_PAGES_XY;
			uint LoopEndXY = Setup.GetLoopEnd(0);
			for (uint PageY = Setup.LoopStart.y; PageY < LoopEndXY; PageY += Setup.LoopStride)
			{
				for (uint PageX = Setup.LoopStart.x; PageX < LoopEndXY; PageX += Setup.LoopStride)
				{
					uint2 Level0Page;
					Level0Page.x = PageX;
					Level0Page.y = PageY;
					// Directional lights propagate pages to their coarser/larger clipmap levels (and only use mip0 pages)
					// Each clipmap level is a separate VSM, so we gather any mapped coarser pages as necessary and write only our own page output
					// There's also technically a race similar to below with other threads writing the PT data we are reading,
					// but it's still deterministic as long as we only look at pages with "bThisLODValid".
					// There's some redundant work of course, but this shader is pretty cheap overall

					FVSMPageOffset Page0Offset = CalcPageOffset(Setup.VirtualShadowMapHandle, 0, Level0Page);
					uint Page0Flags = InOutPageRequestFlags[Page0Offset.GetResourceAddress()];

					uint Mask4x4 = 0u;
					if (ProjectionData.bUseReceiverMask)
					{
						uint2 MaskAddress = Page0Offset.GetResourceAddress() * 2u;
						// 1. load 2x2 mask from level 0
						uint4 Mask8x8 = GatherPageTableRW(InOutPageReceiverMasks, MaskAddress); 
						//   construct 4x4 mask
						Mask4x4 = MipMask8x8(Mask8x8);
					}

					BRANCH
					if (Page0Flags != 0u)
					{
						const int OffsetScale = (VSM_LEVEL0_DIM_PAGES_XY >> 2);
						int2 BaseOffset = OffsetScale * ProjectionData.ClipmapCornerRelativeOffset;
						int2 BasePage   = int2(Level0Page) - BaseOffset;

						// Search for first mapped page past this one
						uint RemainingLevels = ProjectionData.ClipmapLevelCountRemaining;
						for (uint ClipmapOffset = 1; ClipmapOffset < RemainingLevels; ++ClipmapOffset)
						{
							FVirtualShadowMapHandle ClipmapLevelHandle = Setup.VirtualShadowMapHandle.MakeOffset(int(ClipmapOffset));
							FVirtualShadowMapProjectionShaderData LevelProjectionData = GetVirtualShadowMapProjectionData(ClipmapLevelHandle);								
							int2 LevelOffset = OffsetScale * LevelProjectionData.ClipmapCornerRelativeOffset;

							int2 LevelMaskAddress = (BasePage + (LevelOffset << ClipmapOffset)) >> (ClipmapOffset - 1u);
							uint2 FractPage = uint2(LevelMaskAddress) & 1u;
							int2 LevelPage = LevelMaskAddress >> 1u;
							if (IsVirtualShadowMapPageAddressValid(LevelPage, 0))
							{
								FVSMPageOffset LevelPageOffset = CalcPageOffset(ClipmapLevelHandle, 0, uint2(LevelPage));
								uint PrevFlags = 0u;
								uint2 PageTableAddress = LevelPageOffset.GetResourceAddress();
								InterlockedOr(InOutPageRequestFlags[LevelPageOffset.GetResourceAddress()], VSM_FLAG_SECONDARY_REQUEST | VSM_FLAG_DETAIL_GEOMETRY, PrevFlags);
								bool bResultUnchanged = (PrevFlags & VSM_FLAG_SECONDARY_REQUEST) != 0u;

								if (ProjectionData.bUseReceiverMask)
								{
									uint2 PageTableMaskAddress = PageTableAddress * 2u + FractPage;
									uint PrevMask = 0u;
									InterlockedOr(InOutPageReceiverMasks[PageTableMaskAddress], Mask4x4, PrevMask);
									bResultUnchanged = bResultUnchanged && PrevMask == Mask4x4;

									Mask4x4 = MipMask4x4(Mask4x4, FractPage);
								}
								// We didn't change anything, some other thread will propagate upwards
								if (bResultUnchanged)
								{
									break;
								}
							}
						}
					}
				}
			}
		}
	}
};

/**
*/
[numthreads(PER_PAGE_THREAD_GROUP_SIZE_XY, PER_PAGE_THREAD_GROUP_SIZE_XY, 1)]
void PropagateFilterableRequestsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FPerPageDispatchSetup Setup; 
	FRequestPropagator Propagator;
	Setup.Execute(DispatchThreadId, Propagator);
}

#endif // PropagateFilterableRequestsCS
