// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPageMarking.usf: 
=============================================================================*/

#define VSM_WITH_DOF_BIAS 1

#include "VirtualShadowMapPageMarking.ush"
#include "/Engine/Shared/SingleLayerWaterDefinitions.h"
#include "../HairStrands/HairStrandsVisibilityCommonStruct.ush"
#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "../LightGridCommon.ush"
#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "../HairStrands/HairStrandsVisibilityCommon.ush"
#include "../HairStrands/HairStrandsTileCommon.ush"
#include "../Substrate/Substrate.ush"
#include "VirtualShadowMapProjectionDirectional.ush"
#include "VirtualShadowMapProjectionSpot.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapStats.ush"
#include "VirtualShadowMapLightGrid.ush"
#include "../ShaderPrint.ush"
#include "../Froxel/Froxel.ush"
#include "../Nanite/NaniteHZBCull.ush"

// Type of input data consume by the page allocation (i.e., data read from the source buffer: Gbuffer, HairStrands data, ...)
#define INPUT_TYPE_GBUFFER 0
#define INPUT_TYPE_HAIRSTRANDS 1
#define INPUT_TYPE_GBUFFER_AND_WATER_DEPTH 2

StructuredBuffer<uint> ThrottleBuffer;

float PageDilationBorderSizeDirectional;
float PageDilationBorderSizeLocal;
uint InputType;
uint bCullBackfacingPixels;

#ifdef GeneratePageFlagsFromFroxelsCS
int PassId;
#endif

struct FFroxelDebugDrawSetup
{
#if DEBUG_DRAW_GENERATE_FROM_FROXELS
	FShaderPrintContext Context;
	bool bDraw;
#endif
};

uint MakeMask4x4(uint2 Min, uint2 Max)
{
	uint NumXBits = Max.x - Min.x + 1u;
	uint XMask = BitFieldMaskU32(NumXBits, Min.x);

	uint MaxY4 = Max.y << 2u;
	uint MinY4 = Min.y << 2u;
	uint NumYBits = MaxY4 - MinY4 + 4u;
	uint YMask = BitFieldMaskU32(NumYBits, MinY4);
	uint RowMult = 0x1111u & YMask;
	return XMask * RowMult;
}


void MarkPageFroxel(
	FVirtualShadowMapProjectionShaderData ProjectionData,
	bool bUsePageDilation, 
	float2 PageDilationOffset,
	float3 TranslatedWorldPosition,
	FFroxelViewBounds FroxelViewAabb,
	uint MipLevel,
	bool bDirectionalLight,
	FFroxelDebugDrawSetup FroxelDebugDrawSetup)
{
	const bool bIsOrtho = bDirectionalLight;
	const bool bNearClip = !bDirectionalLight;

	uint Flags = VSM_FLAG_ALLOCATED | VSM_FLAG_DETAIL_GEOMETRY | VSM_FLAG_PRIMARY_REQUEST;

	// TODO: Make it possible to get the Nanite view somehow such that we don't have to re-do this stuff, same for invalidation...
	//       Though more optimal TODO is probably to calculate the rect once and then scale & bias to clip levels as needed, more scalar data that way.
	// Go back to clip space
	float4x4 UVToClip;
	UVToClip[0] = float4(2, 0, 0, 0);
	UVToClip[1] = float4(0, -2, 0, 0);
	UVToClip[2] = float4(0, 0, 1, 0);
	UVToClip[3] = float4(-1, 1, 0, 1);

	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);

	float4x4 ViewToShadowTranslatedWorld = PrimaryView.ViewToTranslatedWorld;
	ViewToShadowTranslatedWorld[3].xyz += DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	// Get the culled footprint in the clipmap level
	FFrustumCullData Cull = BoxCullFrustum(
		(FroxelViewAabb.Max + FroxelViewAabb.Min) * 0.5f,
		(FroxelViewAabb.Max - FroxelViewAabb.Min) * 0.5f, 
		ViewToShadowTranslatedWorld,
		mul(ProjectionData.TranslatedWorldToShadowUVMatrix, UVToClip),
		ProjectionData.ShadowViewToClipMatrix,
		bIsOrtho, bNearClip, false /*bSkipFrustumCull*/);

	if (!Cull.bIsVisible)
	{
		return;
	}

	FScreenRect Rect = GetScreenRect(int4(0, 0, VSM_VIRTUAL_MAX_RESOLUTION_XY >> MipLevel, VSM_VIRTUAL_MAX_RESOLUTION_XY >> MipLevel), Cull, 4);
	uint4 RectPages = uint4(Rect.Pixels) >> VSM_LOG2_PAGE_SIZE;

#if DEBUG_DRAW_GENERATE_FROM_FROXELS
	if (FroxelDebugDrawSetup.bDraw)
	{
		float XLoc = 0.3;
		FroxelDebugDrawSetup.Context.Pos.y = 0.13f + 0.02f * float(PassId);
		FroxelDebugDrawSetup.Context.Pos.x = XLoc;
		Print(FroxelDebugDrawSetup.Context, RectPages, FontGreen);
		Newline(FroxelDebugDrawSetup.Context);
		FroxelDebugDrawSetup.Context.Pos.x = XLoc;

		float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
		float3 ShadowTranslatedWorldPosition = TranslatedWorldPosition + ViewToShadowTranslation;
		float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
		ShadowUVz.xyz /= ShadowUVz.w;
		uint MaxVirtualAddress = CalcLevelDimsTexels(MipLevel) - 1U;
		float2 VirtualAddressFloat = ShadowUVz.xy * CalcLevelDimsTexels(MipLevel);

		uint2 VirtualAddress = clamp(uint2(VirtualAddressFloat), 0U, MaxVirtualAddress);
		uint2 PageAddress = VirtualAddress >> VSM_LOG2_PAGE_SIZE;
		Print(FroxelDebugDrawSetup.Context, PageAddress, FontGreen);
	}
#endif

	FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(ProjectionData.VirtualShadowMapHandle, MipLevel);
	for (uint y = RectPages.y; y <= RectPages.w; ++y)
	{
		for (uint x = RectPages.x; x <= RectPages.z; ++x)
		{
			FVSMPageOffset PageFlagOffset = CalcPageOffset(PageTableLevelOffset, MipLevel, uint2(x, y));
			MarkPageAddress(PageFlagOffset, Flags);
		}
	}

	BRANCH
	if (ProjectionData.bUseReceiverMask)
	{
		// Rect in 2x page space storing 4x4 masks in each uint
		uint4 MaskFPRect = uint4(Rect.Pixels) >> (VSM_LOG2_PAGE_SIZE - 1u);
		// Rect in mask space (8x8 per page)
		uint4 MaskRect = uint4(Rect.Pixels) >> (VSM_LOG2_PAGE_SIZE - VSM_LOG2_RECEIVER_MASK_SIZE);
		for (uint y = MaskFPRect.y; y <= MaskFPRect.w; ++y)
		{
			for (uint x = MaskFPRect.x; x <= MaskFPRect.z; ++x)
			{
				// Offset to 4x4 sub mask
				uint2 SubMaskOffset = uint2(x, y);

				// clip the mask rect to the current 4x4
				uint2 LocalMaskMin = clamp(int2(MaskRect.xy) - int2(SubMaskOffset * 4), 0, 3);
				uint2 LocalMaskMax = clamp(int2(MaskRect.zw) - int2(SubMaskOffset * 4), 0, 3);

				uint LocalRectMask = MakeMask4x4(LocalMaskMin, LocalMaskMax);

				if (LocalRectMask != 0u)
				{
					uint2 SubMaskAddress = PageTableLevelOffset.LevelTexelOffset * 2u + uint2(x, y);
					InterlockedOr(OutPageReceiverMasks[SubMaskAddress], LocalRectMask);
				}
			}
		}
	}
}


void MarkPageClipmapFroxel(
	FVirtualShadowMapProjectionShaderData BaseProjectionData,
	bool bUsePageDilation, 
	float2 PageDilationOffset,
	float3 TranslatedWorldPosition,
	FFroxelViewBounds FroxelViewAabb,
	FFroxelDebugDrawSetup FroxelDebugDrawSetup,
	float ExtraBias = 0.0f,
	int MinLevelClamp = 0)
{
	const int ClipmapLevel = max(MinLevelClamp, GetBiasedClipmapLevel(BaseProjectionData, TranslatedWorldPosition, ExtraBias));
	int ClipmapIndex = max(0, ClipmapLevel - BaseProjectionData.ClipmapLevel);
	if (ClipmapIndex < BaseProjectionData.ClipmapLevelCountRemaining)
	{
		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(BaseProjectionData.VirtualShadowMapHandle.MakeOffset(ClipmapIndex));
		MarkPageFroxel(
			ProjectionData, 
			bUsePageDilation, 
			PageDilationOffset, 
			TranslatedWorldPosition, 
			FroxelViewAabb, 
			0u, // MipLevel
			true, // bDirectionalLight
			FroxelDebugDrawSetup);
	}
}

uint MinLocalLightIndex;
uint MaxLocalLightIndex;
int bIncludeMegaLights;
RWStructuredBuffer<uint> OutPrunedLightGridData;
RWStructuredBuffer<uint> OutPrunedNumCulledLightsGrid;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void PruneLightGridCS(uint GridLinearIndex : SV_DispatchThreadID)
{
	uint EyeIndex = View.StereoPassIndex;
	const FLightGridData GridData = GetLightGridData();

	// Ignore NumCulledLightsGrid data that does not represent lights, such as reflection data
	if (GridLinearIndex % ForwardLightStruct.CulledBufferOffsetISR >= GridData.CulledGridSize.x * GridData.CulledGridSize.y * GridData.CulledGridSize.z)
	{
		return;
	}

	const FCulledLightsGridHeader CulledLightGridHeader = GetCulledLightsGridHeader(GridLinearIndex);

	uint NumRetainedLights = 0U;
	uint FirstLightWithVSM = CulledLightGridHeader.NumLights;
	uint LastLightWithVSM = 0U;

	uint NumGridLights = CulledLightGridHeader.NumLights;
	if (bIncludeMegaLights == 0)
	{
		// Exclude megalights if they are being marked in a separate pass
		// TODO: Possibly keep track of where the megalights start in the pruned list?
		// In theory this is recoverable via looking at the global indices though
		NumGridLights = NumGridLights - CulledLightGridHeader.NumMegaLights;
	}

	const uint MaxIndex = min(MaxLocalLightIndex, NumGridLights);

	// First pass add the non-distant lights with VSMs
	LOOP
	for (uint Index = MinLocalLightIndex; Index < MaxIndex; ++Index)
	{
		FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(GetLocalLightDataFromGrid(CulledLightGridHeader.DataStartIndex + Index, EyeIndex).Internal.VirtualShadowMapId);
		if (VirtualShadowMapHandle.IsValid())
		{
			FirstLightWithVSM = min(FirstLightWithVSM, Index);
			LastLightWithVSM = max(LastLightWithVSM, Index);

			if (!VirtualShadowMapHandle.IsSinglePage())
			{
				// Copy light grid data index
				checkStructuredBufferAccessSlow(OutPrunedLightGridData, CulledLightGridHeader.DataStartIndex + NumRetainedLights);
				OutPrunedLightGridData[CulledLightGridHeader.DataStartIndex + NumRetainedLights++] = GetCulledLightDataGrid(CulledLightGridHeader.DataStartIndex + Index);
			}
		}
	}

	// Second pass add the distant lights with VSMs after
	// NOTE: We could add these to the end in the first pass and then re-pack them instead, but this works
	// well enough for moderate light counts.
	LOOP
	for (uint Index = FirstLightWithVSM; Index <= LastLightWithVSM; ++Index)
	{
		FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(GetLocalLightDataFromGrid(CulledLightGridHeader.DataStartIndex + Index, EyeIndex).Internal.VirtualShadowMapId);
		// Discard any light without a VSM
		if (VirtualShadowMapHandle.IsValid() && VirtualShadowMapHandle.IsSinglePage())
		{
			checkStructuredBufferAccessSlow(OutPrunedLightGridData, CulledLightGridHeader.DataStartIndex + NumRetainedLights);
			OutPrunedLightGridData[CulledLightGridHeader.DataStartIndex + NumRetainedLights++] = GetCulledLightDataGrid(CulledLightGridHeader.DataStartIndex + Index);
		}
	}

	checkStructuredBufferAccessSlow(OutPrunedNumCulledLightsGrid, GridLinearIndex);
	OutPrunedNumCulledLightsGrid[GridLinearIndex] = NumRetainedLights;
}

uint2 PixelStride;

#if PERMUTATION_WATER_DEPTH
Texture2D<float> SingleLayerWaterDepthTexture;
StructuredBuffer< uint > SingleLayerWaterTileMask;
int2 SingleLayerWaterTileViewRes;
#endif

#if PERMUTATION_TRANSLUCENCY_DEPTH
uint FrontLayerMode;
float4 FrontLayerHistoryUVMinMax;
float4 FrontLayerHistoryScreenPositionScaleBias;
float4 FrontLayerHistoryBufferSizeAndInvSize;
Texture2D<float> FrontLayerTranslucencyDepthTexture;
Texture2D<float4> FrontLayerTranslucencyNormalTexture;
#endif

struct FPositionData
{
	float SceneDepth;
	float4 SvPosition;
	float3 TranslatedWorldPosition;
	float DeviceZ;
};

FPositionData InitPositionData(uint2 PixelPos, float DeviceZ, bool bIsValid)
{
	FPositionData Out = (FPositionData)0;
	if (bIsValid)
	{
		Out.DeviceZ = DeviceZ;
		Out.SceneDepth = ConvertFromDeviceZ(Out.DeviceZ);
		Out.SvPosition = float4(float2(PixelPos) + 0.5f, Out.DeviceZ, 1.0f);
		Out.TranslatedWorldPosition = SvPositionToTranslatedWorld(Out.SvPosition);
	}
	return Out;
}

// Bias applied to pixels originating from first-person geometry, if negative marking is skipped for these
float FirstPersonPixelRequestBias;
int FirstPersonPixelRequestLevelClamp;

// Note: we use the tile size defined by the water as the group-size - this is needed because the tile mask testing code relies on the size being the same to scalarize efficiently.
[numthreads(SLW_TILE_SIZE_XY, SLW_TILE_SIZE_XY, 1)]
void GeneratePageFlagsFromPixels(
	uint3 InGroupId : SV_GroupID,
	uint  GroupIndex : SV_GroupIndex,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 DispatchThreadId : SV_DispatchThreadID)
{
#if PERMUTATION_INPUT_TYPE == INPUT_TYPE_HAIRSTRANDS
	uint2 GroupId		= InGroupId.xy;
	if (HairStrands.bHairTileValid>0)
	{
		const uint TileCount = HairStrands.HairTileCount[HAIRTILE_HAIR_ALL];
		const uint LinearIndex  = InGroupId.x + InGroupId.y * HairStrands.HairTileCountXY.x;
		if (LinearIndex >= TileCount)
		{
			return;
		}
		GroupId = HairStrands.HairTileData[LinearIndex];
	}
#else // PERMUTATION_INPUT_TYPE == INPUT_TYPE_GBUFFER || PERMUTATION_INPUT_TYPE == INPUT_TYPE_GBUFFER_AND_WATER_DEPTH
	const uint2 GroupId = InGroupId.xy;
#endif
	const uint2 PixelLocalPos = DispatchThreadId.xy * PixelStride;
	const uint2 PixelPos = uint2(View.ViewRectMin.xy) + PixelLocalPos;

	if (any(PixelPos >= uint2(View.ViewRectMin.xy + View.ViewSizeAndInvSize.xy)))
	{
		return;
	}

	half3  WorldNormal				= half3(0, 0, 0);
	bool   bBackfaceCull			= bCullBackfacingPixels != 0;
	float  DeviceZ					= 0;
	float  DeviceZWater				= 0;
	float  DeviceZTranslucency		= 0;
	bool   bIsValid					= true;
	bool   bIsWaterDepthValid		= false;
	bool   bIsTranslucencyDepthValid= false;
	bool   bIsFirstPersonPixel		= false;
#if PERMUTATION_INPUT_TYPE == INPUT_TYPE_HAIRSTRANDS
	{
		DeviceZ = HairStrands.HairOnlyDepthTexture.Load(uint3(PixelPos, 0)).x;
		bIsValid = DeviceZ > 0;
		bBackfaceCull = false;
	}
#else // PERMUTATION_INPUT_TYPE == INPUT_TYPE_GBUFFER
	{
		DeviceZ = LookupDeviceZ(PixelPos);
	#if SUBSTRATE_GBUFFER_FORMAT==1
		const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelPos, 0)));
		FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
		FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
		WorldNormal = TopLayerData.WorldNormal;
		bIsValid = SubstratePixelHeader.IsSubstrateMaterial();
		bBackfaceCull = bBackfaceCull && !SubstratePixelHeader.HasSubsurface();
		bIsFirstPersonPixel = SubstratePixelHeader.IsFirstPerson();
	#else // SUBSTRATE_GBUFFER_FORMAT==1
		FGBufferData GBufferData = GetGBufferDataUint(PixelPos, true);
		WorldNormal = GBufferData.WorldNormal;
		// Excluding unlit to avoid including processing sky dome
		bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
		bBackfaceCull = bBackfaceCull && !IsSubsurfaceModel(GBufferData.ShadingModelID);
		bIsFirstPersonPixel = IsFirstPerson(GBufferData);
	#endif // SUBSTRATE_GBUFFER_FORMAT==1

	#if PERMUTATION_WATER_DEPTH
		// Assume valid if there is no mask
		bIsWaterDepthValid = SingleLayerWaterTileViewRes.x <= 0;
		// Skip the mask pass if the rect is zero
		if (!bIsWaterDepthValid)
		{
			// uint2 WaterTileIndex = DispatchThreadId.xy * PixelStride / SLW_TILE_SIZE_XY; <=>
			// uint2 WaterTileIndex = (groupId*SLW_TILE_SIZE_XY + threadgroupIndex) * PixelStride / SLW_TILE_SIZE_XY; <=>
			// uint2 WaterTileIndex = groupId * PixelStride + threadgroupIndex * PixelStride / SLW_TILE_SIZE_XY; <=>
			// uint2 WaterTileMin = groupId * PixelStride + {0,0} * PixelStride / SLW_TILE_SIZE_XY; <=> groupId * PixelStride
			// uint2 WaterTileMax = groupId * PixelStride + (SLW_TILE_SIZE_XY-1) * PixelStride / SLW_TILE_SIZE_XY; <=> 
			// uint2 WaterTileMax = groupId * PixelStride + (SLW_TILE_SIZE_XY * PixelStride - PixelStride)  / SLW_TILE_SIZE_XY; <=> 
			// uint2 WaterTileMax = groupId * PixelStride + PixelStride - PixelStride / SLW_TILE_SIZE_XY; <~> 
			// uint2 WaterTileMax = groupId * PixelStride + PixelStride; <~> 

			// Bit-scalarized version, must test rect
			uint2 WaterTileMin = GroupId * PixelStride;
			uint2 WaterTileMax = WaterTileMin + PixelStride;

			for (uint WaterTileY = WaterTileMin.y; WaterTileY < WaterTileMax.y && !bIsWaterDepthValid; ++WaterTileY)
			{
				for (uint WaterTileX = WaterTileMin.x; WaterTileX < WaterTileMax.x; ++WaterTileX)
				{
					uint MaskLinearIndex = uint(SingleLayerWaterTileViewRes.x) * WaterTileY + WaterTileX;
					if ((SingleLayerWaterTileMask[MaskLinearIndex / 32U] & (1U << (MaskLinearIndex % 32U))) != 0U)
					{
						bIsWaterDepthValid = true;
						break;
					}
				}
			}
		}
		if (bIsWaterDepthValid)
		{
			DeviceZWater = SingleLayerWaterDepthTexture.Load(uint3(PixelPos, 0)).x;
			bIsWaterDepthValid = DeviceZWater != DeviceZ;
		}
	#endif // PERMUTATION_WATER_DEPTH

	#if PERMUTATION_TRANSLUCENCY_DEPTH
		// Same frame front layer depth
		if (FrontLayerMode == 0)
		{
			DeviceZTranslucency = FrontLayerTranslucencyDepthTexture.Load(uint3(PixelPos, 0)).x;
			const float4 EncodedData = FrontLayerTranslucencyNormalTexture.Load(uint3(PixelPos, 0));
			bIsTranslucencyDepthValid = EncodedData.w > 0 && DeviceZTranslucency > 0;
		}
		// Previous frame Front layer depth reprojection
		else
		{
			const float2 ScreenPosition = SvPositionToScreenPosition(float4(PixelPos, 0, 1)).xy;

			float3 HistoryScreenPosition = float3(ScreenPosition, DeviceZ);
			const float4 ThisClip = float4(HistoryScreenPosition, 1);
			//float4 PrevClip = mul(ThisClip, View.ClipToPrevClip); //<=== doesn't contain AA offsets
			const float4 PrevClip = mul(ThisClip, View.ClipToPrevClipWithAA);
			const float3 PrevScreen = PrevClip.xyz / PrevClip.w;
			const float3 Velocity = HistoryScreenPosition - PrevScreen;
			// TODO handle dynamic object by using velocity buffer?
			HistoryScreenPosition -= Velocity;

			const float2 HistoryUV = HistoryScreenPosition.xy * FrontLayerHistoryScreenPositionScaleBias.xy + FrontLayerHistoryScreenPositionScaleBias.wz;
			bool bHistoryValid = true;
			FLATTEN
			if (any(HistoryUV > FrontLayerHistoryUVMinMax.zw) || any(HistoryUV < FrontLayerHistoryUVMinMax.xy))
			{
				bHistoryValid = false;
			}

			if (bHistoryValid)
			{
				const float2 HistoryPixelPos = HistoryUV * FrontLayerHistoryBufferSizeAndInvSize.xy;
				DeviceZTranslucency = FrontLayerTranslucencyDepthTexture.Load(uint3(HistoryPixelPos, 0)).x;
				const float4 EncodedData = FrontLayerTranslucencyNormalTexture.Load(uint3(PixelPos, 0));
				bIsTranslucencyDepthValid = EncodedData.w > 0 && DeviceZTranslucency > 0;
			}
		}
	#endif
	}
#endif // PERMUTATION_INPUT_TYPE == INPUT_TYPE_GBUFFER
	
	if (!bIsValid && !bIsWaterDepthValid && !bIsTranslucencyDepthValid)
	{
		return;
	}
	
	const FPositionData Primary = InitPositionData(PixelPos, DeviceZ, true);

#if PERMUTATION_WATER_DEPTH
	const FPositionData Water = InitPositionData(PixelPos, DeviceZWater, bIsWaterDepthValid);
#endif
	
#if PERMUTATION_TRANSLUCENCY_DEPTH
	const FPositionData Translucency = InitPositionData(PixelPos, DeviceZTranslucency, bIsTranslucencyDepthValid);
#endif

	// Dither pattern for page dilation
	// We don't need to to check all 8 adjacent pages; as long as there's at least a single pixel near the edge
	// the adjacent one will get mapped. In practice only checking one diagonal seems to work fine and have minimal
	// overhead.
	const float2 PageDilationDither = float2(
		(GroupIndex & 1) ? 1.0f : -1.0f,
		(GroupIndex & 2) ? 1.0f : -1.0f);
		
	// Apply extra bias if the pixel is from first-person geometry, as this geometry is not self-shadowing and very close to the camera (requesting high resolution)
	// We can't outright skip the pixels as we want environment shadow cast onto the FP geometry.
	float PrimaryExtraBias = bIsFirstPersonPixel ? FirstPersonPixelRequestBias : 0.0f;
	int PrimaryMinLevelClamp = bIsFirstPersonPixel ? FirstPersonPixelRequestLevelClamp : -10000;
	float LocalExtraBias = 0.0f;
	
	float DOFResolutionBias = CalculateResolutionBiasFromDepthOfField(Primary.SceneDepth);
	PrimaryExtraBias += DOFResolutionBias;
	LocalExtraBias += DOFResolutionBias;

	bool bMarkFirstPersonPixels = FirstPersonPixelRequestBias >= 0.0f;
	// Directional lights
	{
		const bool bUsePageDilation = PageDilationBorderSizeDirectional > 0.0f;
		const float2 PageDilationOffset = PageDilationBorderSizeDirectional * PageDilationDither;

		for (uint Index = 0; Index < VirtualShadowMap.DirectionalLightIdCount; ++Index)
		{
			FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMap.DirectionalLightIds[Index]);
			FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

			bool bMarkPrimary = true;
			#if PERMUTATION_INPUT_TYPE != INPUT_TYPE_HAIRSTRANDS
			// Backface test if requested
			if (bBackfaceCull && IsBackfaceToDirectionalLight(WorldNormal, ProjectionData.LightDirection, ProjectionData.LightSourceRadius))
			{
				bMarkPrimary = false;
			}
			#endif

			// Don't mark first person pixels in a first person (world space) shadow map as it doesn't cast shadow onto the FP mesh, or if the extra bias is negative (to explictly disable).
			bool bIsFirstPersonShadowMap = (ProjectionData.Flags & VSM_PROJ_FLAG_IS_FIRST_PERSON_SHADOW) != 0u;
			if (bIsFirstPersonPixel && (bIsFirstPersonShadowMap || !bMarkFirstPersonPixels))
			{
				bMarkPrimary = false;
			}

			if (bMarkPrimary)
			{
				MarkPageDirectional(VirtualShadowMapHandle, Primary.TranslatedWorldPosition, bUsePageDilation, PageDilationOffset, PrimaryExtraBias, PrimaryMinLevelClamp);
			}
			
			#if PERMUTATION_WATER_DEPTH
			if (bIsWaterDepthValid)
			{
				MarkPageDirectional(VirtualShadowMapHandle, Water.TranslatedWorldPosition, bUsePageDilation, PageDilationOffset);
			}
			#endif // PERMUTATION_INPUT_TYPE == INPUT_TYPE_GBUFFER_AND_WATER_DEPTH


			#if PERMUTATION_TRANSLUCENCY_DEPTH
			if (bIsTranslucencyDepthValid)
			{
				MarkPageDirectional(VirtualShadowMapHandle,  Translucency.TranslatedWorldPosition, bUsePageDilation, PageDilationOffset);
			}
			#endif
		}
	}

	// Local lights
	// Don't mark pixels originating from first person if disabled.
	if (!bIsFirstPersonPixel || bMarkFirstPersonPixels)
	{
		const bool bUsePageDilation = PageDilationBorderSizeLocal > 0.0f;
		const float2 PageDilationOffset = PageDilationBorderSizeLocal * PageDilationDither;
				
		uint2 LocalPosition = PixelPos - uint2(View.ViewRectMin.xy);
		const FCulledLightsGridHeader LightGridHeader = VirtualShadowMapGetLightsGridHeader(LocalPosition, Primary.SceneDepth);

		LOOP
		for (uint Index = 0; Index < LightGridHeader.NumLights; ++Index)
		{
			const FLocalLightData LightData = VirtualShadowMapGetLocalLightData(LightGridHeader, Index);

			FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(LightData.Internal.VirtualShadowMapId);

			// If we hit a distant light, we're done as we sorted those to the end. See PruneLightGridCS.
			if (VirtualShadowMapHandle.IsSinglePage())
			{
				break;
			}

			// Relative to PrimaryView
			const float3 LightTranslatedWorldPosition = UnpackLightTranslatedWorldPosition(LightData);
			float LengthSquared = length2(LightTranslatedWorldPosition - Primary.TranslatedWorldPosition);

			if (LengthSquared > Square(1.0f / UnpackLightInvRadius(LightData)))
			{
				continue;
			}

			float3 ToLight = normalize(LightTranslatedWorldPosition - Primary.TranslatedWorldPosition);

			// TODO: Can optimize these tests by fusing them together
			// Also do precise cone test, since froxels are pretty coarse at times.
			if (dot(ToLight, UnpackLightDirection(LightData)) < GetLightSpotAngles(LightData).x)
			{
				continue;
			}

			const uint LightSceneInfoExtraDataPacked = UnpackLightSceneInfoExtraDataPacked(LightData);
			const uint LightType = UnpackLightType(LightSceneInfoExtraDataPacked);
			
			const bool bIsRectLight = LightType == LIGHT_TYPE_RECT;
			if (bIsRectLight)
			{
				const bool bIsBehindRectLight = dot(ToLight, UnpackLightDirection(LightData)) < 0;
				if (bIsBehindRectLight)
				{
					continue;
				}
			}

			// TODO: Precise radius test necessary?

		#if PERMUTATION_INPUT_TYPE != INPUT_TYPE_HAIRSTRANDS
			// Backface test if requested
			if (bBackfaceCull && IsBackfaceToLocalLight(ToLight, WorldNormal, GetLightSourceRadius(LightData)))
			{
				continue;
			}
		#endif

			FDeferredLightData Light = ConvertToDeferredLight(LightData);
			MarkPageLocal(Light, VirtualShadowMapHandle, Primary.TranslatedWorldPosition, Primary.SceneDepth, bUsePageDilation, PageDilationOffset, LocalExtraBias);
		}
	}
}

#ifdef GeneratePageFlagsFromFroxelsCS

StructuredBuffer<FPackedFroxel>	Froxels;
Buffer<uint>			FroxelArgs;

int3 LoadFroxel(uint Index)
{
	return UnpackFroxel(Froxels[Index]);
}

FCulledLightsGridHeader VirtualShadowMapGetLightsGridForFroxel(int3 FroxelCoord, float FroxelCenterViewZ)
{
	uint EyeIndex = View.StereoPassIndex;

	// Pick center point in froxel in pixels
	uint2 PixelPos = FroxelCoord.xy * FROXEL_TILE_SIZE + FROXEL_TILE_SIZE / 2;

	// TODO: calculate mapping directly?
	uint GridLinearIndex = ComputeLightGridCellIndex(PixelPos, FroxelCenterViewZ, EyeIndex);
	FCulledLightsGridHeader CulledLightGridData = GetCulledLightsGridHeader(GridLinearIndex);

	// Replace light count with our pruned count
	CulledLightGridData.NumLights = VirtualShadowMap.NumCulledLightsGrid[GridLinearIndex];
	return CulledLightGridData;
}

uint bShouldMarkLocaLights;
float DebugRange;

[numthreads(FROXEL_INDIRECT_ARG_WORKGROUP_SIZE, 1, 1)]
void GeneratePageFlagsFromFroxelsCS(uint FroxelIndex : SV_DispatchThreadID)
{
#if DEBUG_DRAW_GENERATE_FROM_FROXELS
	FShaderPrintContext Context = InitShaderPrintContext();
#endif
	uint FroxelCount = FroxelArgs[3];

	if (FroxelIndex >= FroxelCount)
	{
		return;
	}
	int3 FroxelCoord = LoadFroxel(FroxelIndex);

	float3 FroxelCenterView = GetFroxelViewSpaceCenter(FroxelCoord);
	float3 FroxelCenterTranslatedWorld = mul(float4(FroxelCenterView, 1.0f), View.ViewToTranslatedWorld).xyz;
	FFroxelViewBounds FroxelViewAabb = GetFroxelViewSpaceAABB(FroxelCoord);

#if DEBUG_DRAW_GENERATE_FROM_FROXELS
	if (FroxelIndex == 0)
	{
		float XLoc = 0.2;
		Context.Pos.y = 0.13f + 0.02f * float(PassId);
		Context.Pos.x = XLoc;
		Print(Context, float(FroxelCount) / 1000, FontGreen);
	}
	FFroxelDebugDrawSetup DebugDrawSetup;
	DebugDrawSetup.Context = Context;
	DebugDrawSetup.bDraw = false;
	if (!Context.IsDrawLocked())
	{
		float Range = length(float2(View.CursorPosition / FROXEL_TILE_SIZE) - float2(FroxelCoord.xy));
		if (Range < DebugRange)
		{
			DebugDrawSetup.bDraw = true;
			FFroxelClipBounds ClipBounds = GetFroxelClipBounds(FroxelCoord);
			float Fade = saturate(8.0f * (DebugRange - Range) / DebugRange);
			float4 Color = float4(IntToColor(FroxelCoord.x + FroxelCoord.y * 13791 + FroxelCoord.z * 93113791), Fade);
			//AddOBBTWS(Context, ClipBounds.ClipMin, ClipBounds.ClipMax, Color, View.ClipToTranslatedWorld, true);

			// Draw the center as well
			AddSphereTWS(Context, FroxelCenterTranslatedWorld, 0.1f, Color);
			// And the view space bounds.
			//FFroxelViewBounds ViewBounds = GetFroxelViewSpaceAABB(FroxelCoord);
			AddOBBTWS(Context, FroxelViewAabb.Min, FroxelViewAabb.Max, Color, DFHackToFloat(View.ViewToTranslatedWorld));
		}
	}
#else
	FFroxelDebugDrawSetup DebugDrawSetup = (FFroxelDebugDrawSetup)0;
#endif
	// Dither pattern for page dilation
	// We don't need to to check all 8 adjacent pages; as long as there's at least a single pixel near the edge
	// the adjacent one will get mapped. In practice only checking one diagonal seems to work fine and have minimal
	// overhead.
	const float2 PageDilationDither = float2(
		(FroxelIndex & 1) ? 1.0f : -1.0f,
		(FroxelIndex & 2) ? 1.0f : -1.0f);

	float PrimaryExtraBias = 0.0f;
	float LocalExtraBias = 0.0f;
	
	float DOFResolutionBias = CalculateResolutionBiasFromDepthOfField(FroxelCenterView.z);
	PrimaryExtraBias += DOFResolutionBias;
	LocalExtraBias += DOFResolutionBias;
		
	// Directional lights
	{
		const bool bUsePageDilation = PageDilationBorderSizeDirectional > 0.0f;
		const float2 PageDilationOffset = PageDilationBorderSizeDirectional * PageDilationDither;

		for (uint Index = 0; Index < VirtualShadowMap.DirectionalLightIdCount; ++Index)
		{
			int VirtualShadowMapId = VirtualShadowMap.DirectionalLightIds[Index];
			FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(FVirtualShadowMapHandle::MakeFromIdDirectional(VirtualShadowMapId));

			MarkPageClipmapFroxel(ProjectionData, bUsePageDilation, PageDilationOffset, FroxelCenterTranslatedWorld, FroxelViewAabb, DebugDrawSetup, PrimaryExtraBias);
		}
	}

	// Local lights
	if (bShouldMarkLocaLights != 0)
	{
		const bool bUsePageDilation = PageDilationBorderSizeLocal > 0.0f;
		const float2 PageDilationOffset = PageDilationBorderSizeLocal * PageDilationDither;
				
		const FCulledLightsGridHeader LightGridData = VirtualShadowMapGetLightsGridForFroxel(FroxelCoord, FroxelCenterView.z);

		LOOP
		for (uint Index = 0; Index < LightGridData.NumLights; ++Index)
		{
			const FLocalLightData LightData = VirtualShadowMapGetLocalLightData(LightGridData, Index);
			FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(LightData.Internal.VirtualShadowMapId);

			// If we hit a distant light, we're done as we sorted those to the end. See PruneLightGridCS.
			if (VirtualShadowMapHandle.IsSinglePage())
			{
				break;
			}

			// Relative to PrimaryView
			const float3 LightTranslatedWorldPosition = UnpackLightTranslatedWorldPosition(LightData);
			float3 LocToLight = LightTranslatedWorldPosition - FroxelCenterTranslatedWorld;
			float LengthSquared = length2(LocToLight);

			if (LengthSquared > Square(1.0f / UnpackLightInvRadius(LightData)))
			{
				continue;
			}

			float3 ToLight = normalize(LocToLight);

			// TODO: Can optimize these tests by fusing them together
			// Also do precise cone test, since froxels are pretty coarse at times.
			if (dot(ToLight, UnpackLightDirection(LightData)) < GetLightSpotAngles(LightData).x)
			{
				continue;
			}

			const uint LightSceneInfoExtraDataPacked = UnpackLightSceneInfoExtraDataPacked(LightData);
			const uint LightType = UnpackLightType(LightSceneInfoExtraDataPacked);
			
			const bool bIsRectLight = LightType == LIGHT_TYPE_RECT;
			if (bIsRectLight)
			{
				const bool bIsBehindRectLight = dot(ToLight, UnpackLightDirection(LightData)) < 0;
				if (bIsBehindRectLight)
				{
					continue;
				}
			}

			// TODO: Precise radius test necessary?

			int VirtualShadowMapId = LightData.Internal.VirtualShadowMapId;
			bool bSpotLight = GetLightSpotAngles(LightData).x > -2.0f;
			if( !bSpotLight )
			{
				VirtualShadowMapHandle = VirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(-ToLight));
			}

			FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
			uint MipLevel = GetMipLevelLocal(ProjectionData, FroxelCenterTranslatedWorld, FroxelCenterView.z, LocalExtraBias);

			MarkPageFroxel(ProjectionData, bUsePageDilation, PageDilationOffset, FroxelCenterTranslatedWorld, FroxelViewAabb, MipLevel, false, DebugDrawSetup);
		}
	}
}

#endif 

uint bMarkCoarsePagesLocal;
uint bIncludeNonNaniteGeometry;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void MarkCoarsePages(uint DispatchThreadId : SV_DispatchThreadID)
{
	// Remap thread ID [0..NumShadowMaps) to full / single page ranges.
	uint VirtualShadowMapId = DispatchThreadId;
	if (VirtualShadowMapId < VirtualShadowMap.NumFullShadowMaps)
	{
		VirtualShadowMapId += VSM_MAX_SINGLE_PAGE_SHADOW_MAPS;
	}
	else
	{
		VirtualShadowMapId -= VirtualShadowMap.NumFullShadowMaps;
		if (VirtualShadowMapId >= VirtualShadowMap.NumSinglePageShadowMaps)
		{
			return;
		}
	}
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId);

	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
		
	// NOTE: Coarse pages are very large and tend to get invalidated a lot due to anything in the scene moving
	// Rendering non-nanite geometry into these pages can be very expensive and thus isn't always desirable.
	uint Flags = VSM_FLAG_ALLOCATED | VSM_FLAG_PRIMARY_REQUEST;
				
	if (ProjectionData.bUnreferenced)
	{
		// Don't mark any pages for lights not referenced this render
	}
	else if (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		// Idea here is the clipmaps already cover supersets of lower levels
		// Thus to get coarser pages we can just mark the center page(s) offset by a level/LOD bias
		// The limit on how far dense data goes out from the camera then becomes the world space size of the marked page(s) on the coarses clipmap
		// We could of course mark a broader set of pages in the coarses clipmap level, but the effective radius
		// even from just marking a single one is usually already large enough for the systems that need this
		// data (volumetric fog, translucent light volume).
		if (ProjectionData.bIsCoarseClipLevel)
		{
			// TODO: Optimize this... can be boiled down to be just in terms of the snap offsets
			float3 OriginTranslatedWorld = ProjectionData.ClipmapWorldOriginOffset;
			float4 ShadowUVz = mul(float4(OriginTranslatedWorld, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
			float2 VirtualTexelAddressFloat = ShadowUVz.xy * float(CalcLevelDimsTexels(0));
			float2 PageAddressFloat = VirtualTexelAddressFloat * float(1.0f / VSM_PAGE_SIZE);
			// NOTE: Page addresses round down/truncate normally, so grab the surrounding 4
			int4 PageAddressLowHigh = int4(floor(PageAddressFloat - 0.5f), ceil(PageAddressFloat - 0.5f));

			MarkPageAddress(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.xy), Flags);
			MarkPageAddress(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.xw), Flags);
			MarkPageAddress(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.zy), Flags);
			MarkPageAddress(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.zw), Flags);

			BRANCH
			if (ProjectionData.bUseReceiverMask)
			{
				MarkFullPageReceiverMask(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.xy));
				MarkFullPageReceiverMask(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.xw));
				MarkFullPageReceiverMask(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.zy));
				MarkFullPageReceiverMask(CalcPageOffset(VirtualShadowMapHandle, 0, PageAddressLowHigh.zw));
			}
		}
	}
	// Note: always mark last mip for "distant" light
	else if (bMarkCoarsePagesLocal != 0U || VirtualShadowMapHandle.IsSinglePage())
	{
		// Mark last mip
		FVSMPageOffset PageOffset = CalcPageOffset(VirtualShadowMapHandle, VSM_MAX_MIP_LEVELS - 1, uint2(0, 0));
		MarkPageAddress(PageOffset, Flags);
		BRANCH
		if (ProjectionData.bUseReceiverMask)
		{
			MarkFullPageReceiverMask(PageOffset);
		}
	}
}
