// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPageMarking.ush: 
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../LightData.ush"
#include "VirtualShadowMapProjectionStructs.ush"
#include "VirtualShadowMapProjectionCommon.ush"

// Flags generated by per-pixel pass to determine which pages are required to provide shadow for the visible geometry
RWTexture2D<uint> OutPageRequestFlags;
RWTexture2D<uint> OutPageReceiverMasks;

// NOTE: Returned absolute clipmap level can be negative; these are well defined
int GetBiasedClipmapLevel(FVirtualShadowMapProjectionShaderData BaseProjectionData, float3 TranslatedWorldPosition, float ExtraBias)
{
	#if PERMUTATION_THROTTLING
	float BiasedLevel = CalcAbsoluteClipmapLevel(BaseProjectionData, TranslatedWorldPosition) + VirtualShadowMap.GlobalResolutionLodBias + ExtraBias;

	int ClipmapIndex = max(0, BiasedLevel - BaseProjectionData.ClipmapLevel);
	if (ClipmapIndex < BaseProjectionData.ClipmapLevelCountRemaining)
	{
		const FVirtualShadowMapHandle VSMHandle = BaseProjectionData.VirtualShadowMapHandle.MakeOffset(ClipmapIndex);
		float PerVSMBias = GetVirtualShadowMapProjectionData(VSMHandle).ResolutionLodBias;
		BiasedLevel += PerVSMBias;
	}
	else
	{
		BiasedLevel += BaseProjectionData.ResolutionLodBias;
	}
	#else
	float BiasedLevel = CalcAbsoluteClipmapLevel(BaseProjectionData, TranslatedWorldPosition) + BaseProjectionData.ResolutionLodBias + VirtualShadowMap.GlobalResolutionLodBias + ExtraBias;
	#endif

	return int(floor(BiasedLevel));
}

uint GetMipLevelLocal(FVirtualShadowMapProjectionShaderData ProjectionData, float3 TranslatedWorldPosition, float SceneDepth, /* >= 0 */ float ExtraBias = 0.0f)
{
	// If local lights are near the primary view the combined offset should be small
	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 ShadowTranslatedWorldPosition = TranslatedWorldPosition + ViewToShadowTranslation;

	float Footprint = VirtualShadowMapCalcPixelFootprintLocal(ProjectionData, ShadowTranslatedWorldPosition, SceneDepth);
	return GetMipLevelLocal(Footprint, VirtualShadowMap.MipModeLocal, ProjectionData.ResolutionLodBias, VirtualShadowMap.GlobalResolutionLodBias, ExtraBias);
}

uint GetMipLevelLocal(FVirtualShadowMapHandle VirtualShadowMapHandle, float3 TranslatedWorldPosition, float SceneDepth, /* >= 0 */ float ExtraBias = 0.0f)
{
	return GetMipLevelLocal(GetVirtualShadowMapProjectionData(VirtualShadowMapHandle), TranslatedWorldPosition, SceneDepth, ExtraBias);
}

void MarkPageAddress(FVSMPageOffset PageOffset, uint Flags)
{
	// checkStructuredBufferAccessSlow(OutPageRequestFlags, PageOffset.GetResourceAddress());
	OutPageRequestFlags[PageOffset.GetResourceAddress()] = Flags;
}

void MarkFullPageReceiverMask(FVSMPageOffset PageOffset)
{
	// atomic or the mask onto the approapriate sub-word
	OutPageReceiverMasks[PageOffset.GetResourceAddress() * 2u + uint2(0,0)] = 0xFFFFu;
	OutPageReceiverMasks[PageOffset.GetResourceAddress() * 2u + uint2(1,0)] = 0xFFFFu;
	OutPageReceiverMasks[PageOffset.GetResourceAddress() * 2u + uint2(0,1)] = 0xFFFFu;
	OutPageReceiverMasks[PageOffset.GetResourceAddress() * 2u + uint2(1,1)] = 0xFFFFu;
}

void MarkPageReceiverMask(FVSMPageOffset PageOffset, uint2 VirtualAddress)
{
	// 8x8 address in the mask
	// 4x4 sub mask
	uint2 MaskAddress = (VirtualAddress >> (VSM_LOG2_PAGE_SIZE - VSM_LOG2_RECEIVER_MASK_SIZE)) & VSM_RECEIVER_MASK_SUBMASK;
	// 2x2 quadrant mask
	uint2 MaskQuadrant = (VirtualAddress >> (VSM_LOG2_PAGE_SIZE - 1u) & 1u);
	// atomic or the mask onto the approapriate sub-word
	InterlockedOr(OutPageReceiverMasks[PageOffset.GetResourceAddress() * 2u + MaskQuadrant], 1u << (MaskAddress.y * 4u + MaskAddress.x));
}

void MarkPage(FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel, float3 TranslatedWorldPosition, bool bUsePageDilation, float2 PageDilationOffset)
{
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

	// MarkPage (or mark pixel pages) should never run for a distant light.
	checkSlow(!VirtualShadowMapHandle.IsSinglePage());

	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 ShadowTranslatedWorldPosition = TranslatedWorldPosition + ViewToShadowTranslation;
	float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
	ShadowUVz.xyz /= ShadowUVz.w;

	// Check overlap vs the shadow map space
	// NOTE: XY test not really needed anymore with the precise cone test in the caller, but we'll leave it for the moment
	bool bInClip = ShadowUVz.w > 0.0f && 
		all(and(ShadowUVz.xyz <= ShadowUVz.w,
			ShadowUVz.xyz >= float3(-ShadowUVz.ww, 0.0f)));
	if (!bInClip)
	{
		return;
	}
	// Normal pages marked through pixel processing are not "coarse" and should include "detail geometry" - i.e., all geometry
	uint Flags = VSM_FLAG_ALLOCATED | VSM_FLAG_DETAIL_GEOMETRY | VSM_FLAG_PRIMARY_REQUEST;

	uint MaxVirtualAddress = CalcLevelDimsTexels(MipLevel) - 1U;
	float2 VirtualAddressFloat = ShadowUVz.xy * CalcLevelDimsTexels(MipLevel);
	uint2 VirtualAddress = clamp(uint2(VirtualAddressFloat), 0U, MaxVirtualAddress);
	uint2 PageAddress = VirtualAddress >> VSM_LOG2_PAGE_SIZE;
	FVSMPageOffset PageOffset = CalcPageOffset(VirtualShadowMapHandle, MipLevel, PageAddress);
	MarkPageAddress(PageOffset, Flags);

	BRANCH
	if (ProjectionData.bUseReceiverMask)
	{
		MarkPageReceiverMask(PageOffset, VirtualAddress);
	}

	// PageDilationBorderSize == 0 implies PageDilationOffset.xy == 0
	if (bUsePageDilation)
	{
		uint MaxPageAddress = MaxVirtualAddress >> VSM_LOG2_PAGE_SIZE;
		float2 PageAddressFloat = VirtualAddressFloat / float(VSM_PAGE_SIZE);
		uint2 PageAddress2 = clamp(uint2(PageAddressFloat + PageDilationOffset), 0U, MaxPageAddress);
		FVSMPageOffset PageOffset2 = CalcPageOffset(VirtualShadowMapHandle, MipLevel, PageAddress2);
		if (PageOffset2.GetPacked() != PageOffset.GetPacked())
		{
			MarkPageAddress(PageOffset2, Flags);
		}
		uint2 PageAddress3 = clamp(uint2(PageAddressFloat - PageDilationOffset), 0U, MaxPageAddress);
		FVSMPageOffset PageOffset3 = CalcPageOffset(VirtualShadowMapHandle, MipLevel, PageAddress3);
		if (PageOffset3.GetPacked() != PageOffset.GetPacked())
		{
			MarkPageAddress(PageOffset3, Flags);
		}
	}
}

void MarkPageDirectional(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 TranslatedWorldPosition,
	bool bUsePageDilation = false, 
	float2 PageDilationOffset = float2(0, 0),
	float ExtraBias = 0.0f,
	int MinLevelClamp = -10000)
{
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	const int ClipmapLevel = max(MinLevelClamp, GetBiasedClipmapLevel(ProjectionData, TranslatedWorldPosition, ExtraBias));
	int ClipmapIndex = max(0, ClipmapLevel - ProjectionData.ClipmapLevel);
	if (ClipmapIndex < ProjectionData.ClipmapLevelCountRemaining)
	{
		MarkPage(ProjectionData.VirtualShadowMapHandle.MakeOffset(ClipmapIndex), 0, TranslatedWorldPosition, bUsePageDilation, PageDilationOffset);
	}
}

void MarkPageLocal(
	FDeferredLightData Light,
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 TranslatedWorldPosition,
	float SceneDepth,
	bool bUsePageDilation = false, 
	float2 PageDilationOffset = float2(0, 0),
	float LocalExtraBias = 0.0f)
{
	// For point/rect lights we need to offset to the appropriate face
	if (Light.bRadialLight && !Light.bSpotLight)
	{
		float3 ToLightSq = (Light.TranslatedWorldPosition - TranslatedWorldPosition);
		VirtualShadowMapHandle = VirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(-ToLightSq));
	}

	uint MipLevel = GetMipLevelLocal(VirtualShadowMapHandle, TranslatedWorldPosition, SceneDepth, LocalExtraBias);
	MarkPage(VirtualShadowMapHandle, MipLevel, TranslatedWorldPosition, bUsePageDilation, PageDilationOffset);
}
