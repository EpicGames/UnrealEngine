// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../Common.ush"
#include "../Nanite/NaniteHZBCull.ush"
#include "../Nanite/NaniteDataDecode.ush"
#include "../SceneData.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapPageCacheCommon.ush"

Texture2D<uint> HZBPageTable;
StructuredBuffer<uint4> HZBPageRectBounds;
Texture2D<uint> HZBPageFlags;

uint4 VirtualShadowMapGetPageRect(FScreenRect Rect)
{
	return uint4(Rect.Pixels) >> VSM_LOG2_PAGE_SIZE;
}

uint4 VirtualShadowMapClipPageRect(uint4 RectPages, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel, StructuredBuffer<uint4> PageRectBoundsBuffer)
{
	uint4 Bounds = PageRectBoundsBuffer[VirtualShadowMapHandle.GetDataIndex() * VSM_MAX_MIP_LEVELS + MipLevel];
	return uint4(max(RectPages.xy, Bounds.xy), min(RectPages.zw, Bounds.zw));
}

uint4 VirtualShadowMapGetAllocatedPageRect(FScreenRect Rect, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel)
{
	return VirtualShadowMapClipPageRect(
		VirtualShadowMapGetPageRect(Rect),
		VirtualShadowMapHandle,
		MipLevel,
		VirtualShadowMap.AllocatedPageRectBounds);
}

uint4 VirtualShadowMapGetUncachedPageRect(FScreenRect Rect, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel)
{
	return VirtualShadowMapClipPageRect(
		VirtualShadowMapGetPageRect(Rect),
		VirtualShadowMapHandle,
		MipLevel,
		VirtualShadowMap.UncachedPageRectBounds);
}

FScreenRect VirtualShadowMapClipScreenRect(FScreenRect ScreenRect, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel, StructuredBuffer<uint4> PageRectBoundsBuffer)
{
	// Inclusive bounds for pages, e.g. (1,1,1,1) means one page at offset 1
	uint4 BoundsPages = PageRectBoundsBuffer[VirtualShadowMapHandle.GetDataIndex() * VSM_MAX_MIP_LEVELS + MipLevel];
	// scale the bounds to texels (still inclusive)
	int4 Bounds = uint4(BoundsPages.xy << VSM_LOG2_PAGE_SIZE, (BoundsPages.zw << VSM_LOG2_PAGE_SIZE) + VSM_PAGE_SIZE - 1u);
	FScreenRect Result = ScreenRect;
	Result.Pixels = int4(max(ScreenRect.Pixels.xy, Bounds.xy), min(ScreenRect.Pixels.zw, Bounds.zw));
	return Result;
}

/**
 * Same but returns the clamped pixel rect rather than the page rect.
 */
FScreenRect VirtualShadowMapGetUncachedScreenRect(FScreenRect ScreenRect, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel)
{
	return VirtualShadowMapClipScreenRect(ScreenRect, VirtualShadowMapHandle, MipLevel, VirtualShadowMap.UncachedPageRectBounds);
}

uint4 GatherPageFlags(uint2 TexelCoord, uint HMipLevel)
{
	return GatherPageTable(VirtualShadowMap.PageFlags, TexelCoord, HMipLevel, VirtualShadowMap.PageTableTextureSizeInvSize.zw);
}

uint4 GatherReceiverMask(uint2 TexelCoord, uint HMipLevel)
{
	// Note: UV scaled by extra 0.5f to account for 2x size of mask texture
	return GatherPageTable(VirtualShadowMap.PageReceiverMasks, TexelCoord, HMipLevel, VirtualShadowMap.PageReceiverMaskTextureSizeInvSize.zw);
}

bool TestPageFlagMask(uint PageFlags, uint PageFlagMask)
{
	// This mask consists of flags that we are testing for if any are set.
	uint AnyMask = PageFlagMask & VSM_FLAG_TEST_ANY_MASK;
	if ((PageFlags & AnyMask) == 0u)
	{
		return false;
	}
	// sub-mask for flag types where all must be set to pass
	uint AllMask = PageFlagMask & VSM_FLAG_TEST_ALL_MASK;
	return (PageFlags & AllMask) == AllMask;
}

// Returns true if ANY flags in FlagMask are set on at least one overlapped page (see GetPageFlagMaskForRendering below)
// NOTE: Only VSM_FLAG_* are allowed to be used here, not VSM_EXTENDED_*, as this does a hierarchical page flag lookup
// If bDetailGeometry is set, additionally tests whether at least one page has VSM_FLAG_DETAIL_GEOMETRY marked on it
bool OverlapsAnyValidPage(FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel, uint4 RectPages, uint InFlagMask, bool bDetailGeometry)
{
	// Skip empty rectangles (inclusive).
	if (any(RectPages.zw < RectPages.xy))
	{
		return false;
	}

	uint FlagMask = InFlagMask | (bDetailGeometry ? VSM_FLAG_DETAIL_GEOMETRY : 0u);

	uint HMipLevel = MipLevelForRect(RectPages, 2);

	// Calculate the offset in the root mip-level (to support UV based Gather)
	FVSMPageOffset VSMPageOffset = CalcPageOffset(VirtualShadowMapHandle, MipLevel, RectPages.xy);
	uint4 PageFlags2x2 = GatherPageFlags(VSMPageOffset.TexelAddress, HMipLevel);

	// Scale to rect of up-to 2x2 size.
	RectPages >>= HMipLevel;
	PageFlags2x2.yz = (RectPages.x == RectPages.z) ? 0u : PageFlags2x2.yz;	// Mask off right pixels, if footprint is only one pixel wide.
	PageFlags2x2.xy = (RectPages.y == RectPages.w) ? 0u : PageFlags2x2.xy;	// Mask off bottom pixels, if footprint is only one pixel tall.

	// Merge the adjacent flags
	uint PageFlags = PageFlags2x2.x | PageFlags2x2.y | PageFlags2x2.z | PageFlags2x2.w;
	return TestPageFlagMask(PageFlags, FlagMask);
}

bool IntersectMask8x8(uint4 Mask2x2, uint2 Min, uint2 Max)
{
	// Assume: clamped to 8x8 rect

	// general idea: work on 8x8 and pull out relevant sub-ranges.

	// Make an 8-bit row mask & then select bits from that, easier than figuring out the ranges for each sub-range 
	uint NumXBits8 = Max.x - Min.x + 1u;
	uint XMask8 = BitFieldMaskU32(NumXBits8, Min.x);
	// XLow half (left half if you like)
	uint XMaskLow = XMask8 & 0xFu;
	uint XMaskHigh = XMask8 >> 4u;

	// make an 8-row mask
	uint MaxY4 = Max.y << 2u;
	uint MinY4 = Min.y << 2u;
	uint NumYBits = MaxY4 - MinY4 + 1u; // +1u instead of 4 to avoid overflow - we only use it to mask off 0x1111 anyway so no need to mask the last few
	uint YMask8 = BitFieldMaskU32(NumYBits, MinY4);
	uint RowMultLow  = 0x1111u & YMask8;
	uint RowMultHigh = 0x1111u & (YMask8 >> 16u);

	// x: (-, +) y: (+, +) z: (+, -) w: (-, -)

	uint4 TestMask;
	TestMask.w = XMaskLow * RowMultLow;
	TestMask.x = XMaskLow * RowMultHigh;
	TestMask.z = XMaskHigh * RowMultLow;
	TestMask.y = XMaskHigh * RowMultHigh;

	return (Mask2x2.x & TestMask.x) != 0u
		|| (Mask2x2.y & TestMask.y) != 0u
		|| (Mask2x2.z & TestMask.z) != 0u
		|| (Mask2x2.w & TestMask.w) != 0u;
}

/**
 * Expectes a pixel rect clamped to the valid page region but still in pixels rather than pages.
 */
bool OverlapsAnyValidPage(FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel, FScreenRect ClampedScreenRect, uint InFlagMask, bool bDetailGeometry, bool bUseReceiverMask)
{
	// Skip empty rectangles (inclusive).
	if (any(ClampedScreenRect.Pixels.zw < ClampedScreenRect.Pixels.xy))
	{
		return false;
	}
	uint4 RectPages = VirtualShadowMapGetPageRect(ClampedScreenRect); 
	uint HMipLevel = MipLevelForRect(RectPages, 2);

	uint FlagMask = InFlagMask | (bDetailGeometry ? VSM_FLAG_DETAIL_GEOMETRY : 0u);

	// Calculate the offset in the root mip-level (to support UV based Gather)
	FVSMPageOffset VSMPageOffset = CalcPageOffset(VirtualShadowMapHandle, MipLevel, RectPages.xy);
	uint4 PageFlags2x2 = GatherPageFlags(VSMPageOffset.TexelAddress, HMipLevel);

	// Scale to rect of up-to 2x2 size.
	uint4 UnitRect = RectPages >> HMipLevel;
	PageFlags2x2.yz = (UnitRect.x == UnitRect.z) ? 0u : PageFlags2x2.yz;	// Mask off right pixels, if footprint is only one pixel wide.
	PageFlags2x2.xy = (UnitRect.y == UnitRect.w) ? 0u : PageFlags2x2.xy;	// Mask off bottom pixels, if footprint is only one pixel tall.

	// Merge the adjacent flags
	uint PageFlags = PageFlags2x2.x | PageFlags2x2.y | PageFlags2x2.z | PageFlags2x2.w;
	if (TestPageFlagMask(PageFlags, FlagMask))
	{
		BRANCH
		if (bUseReceiverMask)
		{
			// Rect in 2x page space storing 4x4 masks in each uint
			uint4 MaskFPRect = uint4(ClampedScreenRect.Pixels) >> (VSM_LOG2_PAGE_SIZE - 1u);
			uint MaskHMipLevel = MipLevelForRect(int4(MaskFPRect), 2);
			// Mask address in root level, offset by mask fractional position, VSMPageOffset accounts for mip translation so we need that for the base address.
			uint2 MaskAddress = VSMPageOffset.TexelAddress * 2u + (MaskFPRect.xy & 1u);

			// offset in the destination mip level (where we will sample the 2x2 bitmask) - truncate so we can...
			uint2 OffsetMip = MaskFPRect.xy >> MaskHMipLevel;
			// offset back in full bit-mask-res
			uint2 Offset = OffsetMip << (VSM_LOG2_RECEIVER_MASK_SIZE + MaskHMipLevel - 1u);
			// ...figure out the test mask in the right rect
			uint4 MaskRect = uint4(ClampedScreenRect.Pixels) >> (VSM_LOG2_PAGE_SIZE - VSM_LOG2_RECEIVER_MASK_SIZE);
			uint4 MaskSubRect = MaskRect - Offset.xyxy;
			uint4 MaskSubRectMip = MaskSubRect >> MaskHMipLevel;

			uint4 ReceiverMask8x8 = GatherReceiverMask(MaskAddress, MaskHMipLevel);
			return IntersectMask8x8(ReceiverMask8x8, MaskSubRectMip.xy, MaskSubRectMip.zw);
		}
		return true;
	}

	return false;
}

/**
 * Wrapper type to make misuse slightly harder.
 */
struct FPageTestScreenRect
{
	FScreenRect ScreenRect;
	uint HZBLevelPageSizeShift;
	int HZBLevelPageSizeInclusive;
	uint4 RectPages;
	bool bWasPageRectClipped;
};


/**
 * Set up a screen rect and pre-computed data for testing pages against HZB, this assumes a 4x4-HZB FScreenRect
 * as input. The resulting rect has been clamped to the mip level where a page is 4x4 texels, as higher mips are meaningless.
 */
FPageTestScreenRect SetupPageHZBRect(FScreenRect ScreenRect, FVirtualShadowMapHandle VirtualShadowMapHandle, uint MipLevel)
{
	FPageTestScreenRect Result;
	Result.ScreenRect = ScreenRect;
	// Clamp to level where a page is 4x4 (HZB mip 0 is half-size)
	if (Result.ScreenRect.HZBLevel > (VSM_LOG2_PAGE_SIZE - 3))
	{
		// Adjust HZB texel rect to match new mip level, this will be too large, but is clipped below.
		Result.ScreenRect.HZBTexels = int4(Result.ScreenRect.Pixels.xy, max(Result.ScreenRect.Pixels.xy, Result.ScreenRect.Pixels.zw)) >> (VSM_LOG2_PAGE_SIZE - 2U);
		Result.ScreenRect.HZBLevel = VSM_LOG2_PAGE_SIZE - 3U;
	}
	Result.HZBLevelPageSizeShift = VSM_LOG2_PAGE_SIZE - 1U - Result.ScreenRect.HZBLevel;
	Result.HZBLevelPageSizeInclusive = (1U << Result.HZBLevelPageSizeShift) - 1;
	uint4 UnClippedRectPages = VirtualShadowMapGetPageRect(ScreenRect);
	// If the clipped page rect is smaller than the unclipped rect, there are unmapped pages in the footprint and we return 
	// that it is visible.
	Result.RectPages = VirtualShadowMapClipPageRect(UnClippedRectPages, VirtualShadowMapHandle, MipLevel, HZBPageRectBounds);
	Result.bWasPageRectClipped = any(Result.RectPages.xy > UnClippedRectPages.xy) || any(Result.RectPages.zw < UnClippedRectPages.zw);

	return Result;
}


bool IsPageVisibleHZB(uint2 vPage, FVSMPageOffset PageFlagOffset, bool bTreatUnmappedAsOccluded, FPageTestScreenRect PageTestScreenRect, bool bUseStaticOcclusion)
{
	FShadowPhysicalPage pPage = ShadowDecodePageTable(HZBPageTable[PageFlagOffset.GetResourceAddress()]);

	if (pPage.bThisLODValid)
	{
		uint2 PhysicalAddress = pPage.PhysicalAddress;

		FScreenRect HZBTestRect = PageTestScreenRect.ScreenRect;

		// Move to page local (in mip level) space and clamp rect to page size.
		HZBTestRect.HZBTexels -= (vPage << PageTestScreenRect.HZBLevelPageSizeShift).xyxy;
		HZBTestRect.HZBTexels = clamp(HZBTestRect.HZBTexels, 0, PageTestScreenRect.HZBLevelPageSizeInclusive);
		// Translate to physical address space
		HZBTestRect.HZBTexels += (PhysicalAddress << PageTestScreenRect.HZBLevelPageSizeShift).xyxy;

		return IsVisibleHZBArray(HZBTestRect, true, GetVirtualShadowMapHZBArrayIndex(bUseStaticOcclusion));
	}

	return !bTreatUnmappedAsOccluded;
}


/**
 * Perform HZB-Test for a rectangle of pages, returning true if the Rect is visible in at least one page.
 * @param TestPageMask - page flags to inlcude in occlusion test, e.g., VSM_FLAG_ALLOCATED will test any allocated page, but ignore unallocated ones.
 * @param VisiblePageMask - page flags to treat as un-occluded, tested after the above mask, e.g., use VSM_UNCACHED_FLAG to treat any uncached page as visible. 
 */
bool IsVisibleMaskedHZB(FVirtualShadowMapHandle PrevShadowMapHandle, uint MipLevel, FScreenRect Rect, bool bClampToPageLevel, bool bTreatUnmappedAsOccluded, uint VisiblePageMask, uint TestPageMask, bool bUseStaticOcclusion)
{
	// Don't have an HZB to test.
	if (!PrevShadowMapHandle.IsValid())
	{
		return true;
	}

	// Don't go past mip level of 4x4 for a 4x4 test without possibly covering more than 4 pages.
	if (!bClampToPageLevel && Rect.HZBLevel > VSM_LOG2_PAGE_SIZE - 3)
	{
		return true;
	}

	FPageTestScreenRect HZBTestRect = SetupPageHZBRect(Rect, PrevShadowMapHandle, MipLevel);

	// Allow treating unmapped pages as visible, such that 
	if (!bTreatUnmappedAsOccluded && HZBTestRect.bWasPageRectClipped)
	{
		return true;
	}


	uint4 RectPages = HZBTestRect.RectPages;
	FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(PrevShadowMapHandle, MipLevel);

	for (uint y = RectPages.y; y <= RectPages.w; y++)
	{
		for (uint x = RectPages.x; x <= RectPages.z; x++)
		{
			FVSMPageOffset PageFlagOffset = CalcPageOffset(PageTableLevelOffset, MipLevel, uint2(x, y));
			uint PageFlag = HZBPageFlags[PageFlagOffset.GetResourceAddress()];

			// Skip unallocated pages if bTreatUnmappedAsOccluded is true, otherwise test everything
			if (!bTreatUnmappedAsOccluded || ((PageFlag & TestPageMask) != 0U))
			{
				// Treat pages with the VisiblePageMask as visible - can be used to select only cached pages
				if ((PageFlag & VisiblePageMask) != 0U || IsPageVisibleHZB(uint2(x, y), PageFlagOffset, bTreatUnmappedAsOccluded, HZBTestRect, bUseStaticOcclusion))
				{
					return true;
				}
			}
		}
	}

	return false;
}


/**
 * Perform HZB-Test for a rectangle of pages, returning true if the Rect is visible in at least one page.
 * @param TestPageMask - page flags to inlcude in occlusion test, e.g., VSM_FLAG_ALLOCATED will test any allocated page, but ignore unallocated ones.
 * @param VisiblePageMask - page flags to treat as un-occluded, tested after the above mask, e.g., use VSM_UNCACHED_FLAG to treat any uncached page as visible. 
 */
bool IsVisibleMaskedHZB(uint PrevShadowMapID, uint MipLevel, FScreenRect Rect, bool bClampToPageLevel, bool bTreatUnmappedAsOccluded, uint VisiblePageMask, uint TestPageMask, bool bUseStaticOcclusion)
{
	return IsVisibleMaskedHZB(FVirtualShadowMapHandle::MakeFromId(PrevShadowMapID), MipLevel, Rect, bClampToPageLevel, bTreatUnmappedAsOccluded, VisiblePageMask, TestPageMask, bUseStaticOcclusion);
}

uint GetPageFlagMaskForRendering(bool bCacheAsStatic, uint InstanceId, int SceneRendererPrimaryViewId)
{
	uint PageFlagMask = (bCacheAsStatic ?  VSM_FLAG_STATIC_UNCACHED : VSM_FLAG_DYNAMIC_UNCACHED);
	// If it is cached as static & it has just transitioned from being uncached, then use the ALLOCATED_FLAG to make sure it renders despite the page not being invalidated.
	if (bCacheAsStatic)
	{
		if (GetCacheTransitioned(InstanceId, SceneRendererPrimaryViewId))
		{
			return VSM_FLAG_ALLOCATED | VSM_FLAG_DYNAMIC_UNCACHED; 
		}
	}
	return PageFlagMask;
}

float CalcClipSpaceRadiusEstimate(bool bIsOrtho, FInstanceSceneData InstanceData, float4x4 LocalToTranslatedWorld, float4x4 ViewToClip)
{
	float WorldSpaceRadiusLength = length(InstanceData.LocalBoundsExtent * InstanceData.NonUniformScale.xyz);
	if (bIsOrtho)
	{
		// for ortho we just need the estimated radius & the projection scale, which is symmetric for VSM views
		return WorldSpaceRadiusLength * ViewToClip[0][0];
	}
	else
	{
		const float3 TranslatedWorldCenter = mul(float4(InstanceData.LocalBoundsCenter, 1.0f), LocalToTranslatedWorld).xyz;
		float4 RadiusClipH = mul(float4(WorldSpaceRadiusLength, 0.0f, length(TranslatedWorldCenter), 1.0f), ViewToClip);
		return abs(RadiusClipH.x / RadiusClipH.w);
	}
}

/**
 * Figure out the coarse page flag for a given footprint pixel-radius of an instance (estimated from bounds).
 * IF the footprint is considered "small" and thus a candidate for not rendering to coarse pages, we return true - which means it will get compared against the same flag in the page flags in the footprint.
 * When the comparison is not 1 (i.e., for a page _not_ marked with VSM_FLAG_DETAIL_GEOMETRY) the instance is culled.
 * This test should only be performed at the instance level, as it does not make sense to partially cull geometry, (e.g, creating holes based on varying cluster size).
 */
bool IsDetailGeometry(bool bCacheAsStatic, bool bIsNaniteGeometry, float InstancePixelEstRadius)
{
	// Preserve the old behaviour, only based on whether it is nanite or not.
	if (VirtualShadowMap.bExcludeNonNaniteFromCoarsePages)
	{
		return !bIsNaniteGeometry;
	}

	// We use a separate threshold for dynamically cached geometry, this is because the overhead is lower when caching is enabled as cached page culling removes the instances anyway.
	// Nanite geometry has a separate smaller footprint as it has better LOD and lower overhead for replicated drawing.
	if (bCacheAsStatic)
	{
		return InstancePixelEstRadius < VirtualShadowMap.CoarsePagePixelThresholdStatic;
	}

	if (bIsNaniteGeometry)
	{
		// Use a separate threshold for Nanite, as the overhead for drawing small instances is far lower.
		return InstancePixelEstRadius < VirtualShadowMap.CoarsePagePixelThresholdDynamicNanite;
	}

	return InstancePixelEstRadius < VirtualShadowMap.CoarsePagePixelThresholdDynamic;
}


RWStructuredBuffer<uint> OutDirtyPageFlags;

// NOTE: If you add flags here the encoding in PackJob in VirtualShadowMapPhysicalPageManagement.usf may also need to be updated
#define VSM_MARK_PAGE_DIRTY_FLAG_DYNAMIC				1
#define VSM_MARK_PAGE_DIRTY_FLAG_STATIC					2
#define VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_DYNAMIC 	4
#define VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_STATIC 		8
#define VSM_MARK_PAGE_DIRTY_FLAG_WPO_ALLOWED			16
#define VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_LOD_DELTA	32

uint VirtualShadowMapGetMarkPageDirtyFlags(
	bool bCacheAsStatic,
	bool bInvalidatePage,
	bool bWPOAllowed,
	bool bInvalidateLODDelta)
{
	uint Flags = bCacheAsStatic ? VSM_MARK_PAGE_DIRTY_FLAG_STATIC : VSM_MARK_PAGE_DIRTY_FLAG_DYNAMIC;
	if (bInvalidatePage)
	{
		Flags |= bCacheAsStatic ? VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_STATIC : VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_DYNAMIC;
	}
	else if (bWPOAllowed)
	{
		Flags |= VSM_MARK_PAGE_DIRTY_FLAG_WPO_ALLOWED;
	}
	if (bInvalidateLODDelta)
	{
		// Additive bit: independent of the must-respect invalidate path; budget-gated downstream.
		Flags |= VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_LOD_DELTA;
	}
	return Flags;
}

bool VirtualShadowMapMarkPageDirty(
	FVSMPageOffset PageFlagOffset,
	uint MarkPageDirtyFlags)
{
	FShadowPhysicalPage PhysPage = ShadowGetPhysicalPage(PageFlagOffset);
	if (PhysPage.bThisLODValid)
	{
		// Mark the page dirty so we regenerate HZB, etc.
		uint PhysPageIndex = VSMPhysicalPageAddressToIndex(PhysPage.PhysicalAddress);
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_DYNAMIC)
		{
			OutDirtyPageFlags[PhysPageIndex] = 1U;
		}
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_STATIC)
		{
			OutDirtyPageFlags[(VirtualShadowMap.MaxPhysicalPages * 1U) + PhysPageIndex] = 1U;
		}
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_DYNAMIC)
		{
			OutDirtyPageFlags[(VirtualShadowMap.MaxPhysicalPages * 2U) + PhysPageIndex] = 1U;
		}
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_STATIC)
		{
			OutDirtyPageFlags[(VirtualShadowMap.MaxPhysicalPages * 3U) + PhysPageIndex] = 1U;
		}
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_WPO_ALLOWED)
		{
			OutDirtyPageFlags[(VirtualShadowMap.MaxPhysicalPages * 4U) + PhysPageIndex] = 1U;
		}
		if (MarkPageDirtyFlags & VSM_MARK_PAGE_DIRTY_FLAG_INVALIDATE_LOD_DELTA)
		{
			OutDirtyPageFlags[(VirtualShadowMap.MaxPhysicalPages * 5U) + PhysPageIndex] = 1U;
		}
		return true;
	}
	return false;
}
