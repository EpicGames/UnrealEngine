// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "../Common.ush"

#ifdef PER_PAGE_DISPATCH_SETUP

StructuredBuffer<uint> VirtualShadowMapIds;
uint NumVirtualShadowMapIds;
uint VirtualShadowMapIdsOffset;
uint PerPageDispatchDimXY;
uint bUseThreadPerId;

/**
 * Helper to set up the per-thread indexing to process all relevant pages in the page table.
 */
struct FPerPageDispatchSetup
{
	bool bValid;
	FVirtualShadowMapHandle VirtualShadowMapHandle;
	
	uint2 LoopStart;
	uint LoopStride;

	uint PageStartOffset;
	uint PageEndOffset;
	uint MipLevelStart;
	uint MipLevelEnd;

	void Init(uint3 DispatchThreadId)
	{
		bValid = false;
		VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeInvalid();
		if (bUseThreadPerId)
		{
			// regular loop, one thread does all the work
			LoopStart = uint2(0u, 0u);
			LoopStride = 1;

			uint LinearThreadId = DispatchThreadId.y * PerPageDispatchDimXY + DispatchThreadId.x;
			if (LinearThreadId < NumVirtualShadowMapIds)
			{
				bValid = true;
				VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapIds[VirtualShadowMapIdsOffset + LinearThreadId]);
			}
		}
		else
		{
			bValid = true;
			VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapIds[VirtualShadowMapIdsOffset + DispatchThreadId.z]);

			// Dispatch-stride loop
			LoopStart = DispatchThreadId.xy;
			LoopStride = PerPageDispatchDimXY; // e.g. 
		}
		
		if (VirtualShadowMapHandle.IsSinglePage())
		{
			MipLevelStart = VSM_MAX_MIP_LEVELS - 1u;
			MipLevelEnd = VSM_MAX_MIP_LEVELS;
		}
		else
		{
			const FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
			MipLevelStart = ProjectionData.MinMipLevel;
			MipLevelEnd = VSM_MAX_MIP_LEVELS;
			if (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
			{
				MipLevelEnd = 1u;
			}
		}
	}

	uint GetLoopEnd(uint MipLevel)
	{
		return VSM_LEVEL0_DIM_PAGES_XY >> MipLevel;
	}

	template <typename FWorkType>
	void ExecuteInternal(inout FWorkType Work)
	{
		if (VirtualShadowMapHandle.IsSinglePage())
		{
			MipLevelStart = VSM_MAX_MIP_LEVELS - 1u;
			MipLevelEnd = VSM_MAX_MIP_LEVELS;
		}
		else
		{
			const FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
			MipLevelStart = ProjectionData.MinMipLevel;
			MipLevelEnd = VSM_MAX_MIP_LEVELS;

			if (ProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
			{
				MipLevelEnd = 1u;
			}
		}
		Work.Run(this);
	}

	template <typename FWorkType>
	void Execute(uint3 DispatchThreadId, inout FWorkType Work)
	{
		bValid = false;
		VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeInvalid();
		if (bUseThreadPerId)
		{
			// regular loop, one thread does all the work
			LoopStart = uint2(0u, 0u);
			LoopStride = 1;

			uint LinearThreadId = DispatchThreadId.y * PerPageDispatchDimXY + DispatchThreadId.x;
			if (LinearThreadId < NumVirtualShadowMapIds)
			{
				VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapIds[VirtualShadowMapIdsOffset + LinearThreadId]);
				ExecuteInternal(Work);
			}
			return;
		}

		VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapIds[VirtualShadowMapIdsOffset + DispatchThreadId.z]);

		// Dispatch-stride loop
		LoopStart = DispatchThreadId.xy;
		LoopStride = PerPageDispatchDimXY; // e.g. 
	
		ExecuteInternal(Work);
	}

};

#endif
