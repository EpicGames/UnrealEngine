// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapPhysicalPageManagement.usf: 
=============================================================================*/

#include "../Common.ush"
#include "../WaveOpUtil.ush"
#include "../ReductionCommon.ush"
#include "../GPUMessaging.ush"
#include "../ShaderPrint.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "VirtualShadowMapProjectionStructs.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapStats.ush"
#include "VirtualShadowMapPerPageDispatch.ush"

#ifndef HAS_CACHE_DATA
#define HAS_CACHE_DATA 1
#endif //HAS_CACHE_DATA

// Page flags generated by page allocation to indicate state to rendering passes (i.e., present / invalid)
Texture2D<uint> PageRequestFlags;
RWTexture2D<uint> OutPageFlags;
RWTexture2D<uint> OutPageTable;
RWStructuredBuffer<FPhysicalPageMetaData> OutPhysicalPageMetaData;

// A series of lists used to track various page states (free, used)
// Each list is MaxPhysicalPages + 1 uint counter
RWStructuredBuffer<int> OutPhysicalPageLists;

// Stores available pages (i.e. ones not used this frame) for allocation in LRU order
#define PHYSICAL_PAGE_LIST_LRU 0
// Packed available list
// Pages invalidated this frame will be added to the end. Allocations come from the end.
#define PHYSICAL_PAGE_LIST_AVAILABLE 1
// Stores invalidated/empty pages temporarily before they are re-added to the AVAILABLE list
#define PHYSICAL_PAGE_LIST_EMPTY 2
// Stores pages requested/used this frame, not available for allocation
#define PHYSICAL_PAGE_LIST_REQUESTED 3
// Number of page lists
#define PHYSICAL_PAGE_LIST_COUNT 4

int GetPhysicalPageListStart(int PageList)
{
	return PageList * (VirtualShadowMap.MaxPhysicalPages + 1);
}

int GetPhysicalPageListItem(uint PageList, int Index)
{
	return OutPhysicalPageLists[GetPhysicalPageListStart(PageList) + Index];
}

void SetPhysicalPageListItem(uint PageList, int Index, int Value)
{
	//check(Index < VirtualShadowMap.MaxPhysicalPages);
	OutPhysicalPageLists[GetPhysicalPageListStart(PageList) + Index] = Value;
}

int GetPhysicalPageListCount(int PageList)
{
	return OutPhysicalPageLists[GetPhysicalPageListStart(PageList) + VirtualShadowMap.MaxPhysicalPages];
}

void SetPhysicalPageListCount(int PageList, int NewCount)
{
	OutPhysicalPageLists[GetPhysicalPageListStart(PageList) + VirtualShadowMap.MaxPhysicalPages] = NewCount;
}

bool PushPhysicalPageList(uint PageList, int PhysicalPageIndex)
{
	uint PageListStart = GetPhysicalPageListStart(PageList);
	// NOTE: Counter is the final element of the list
	int Offset = 0;
	WaveInterlockedAddScalar_(OutPhysicalPageLists[PageListStart + VirtualShadowMap.MaxPhysicalPages], 1, Offset);
	// We have to guard against overflow as it will overwrite the counter and potentially into other lists
	if (Offset < VirtualShadowMap.MaxPhysicalPages)
	{
		OutPhysicalPageLists[PageListStart + Offset] = PhysicalPageIndex;
		return true;
	}
	else
	{
		return false;
	}
}

// Returns <0 if none available, otherwise returns the actual value
int PopPhysicalPageList(uint PageList)
{
	uint PageListStart = GetPhysicalPageListStart(PageList);
	int Offset = 0;

#if 1
	WaveInterlockedAddScalar_(OutPhysicalPageLists[PageListStart + VirtualShadowMap.MaxPhysicalPages], -1, Offset);
#else
	// Need negative numbers here...
	InterlockedAdd(OutPhysicalPageLists[PageListStart + VirtualShadowMap.MaxPhysicalPages], -1, Offset);
#endif
	 
	// We want the value *after* decrement in this case
	--Offset;
	return Offset < 0 ? INDEX_NONE : OutPhysicalPageLists[PageListStart + Offset];
}

StructuredBuffer<int> PrevPhysicalPageLists;

RWStructuredBuffer<uint4> OutUncachedPageRectBounds;
RWStructuredBuffer<uint4> OutAllocatedPageRectBounds;
uint NumPageRectsToClear;

// This is admitadly a weird fusion of several initializations but it is the first thing
// we run in a given analysis phase so it's more efficient to do it all here rather than
// have several small passes later.
[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void InitPageRectBounds(uint3 Index : SV_DispatchThreadID)
{
	// The X thread id maps to the range NumPageRectsToClear which is (GetNumFullShadowMaps() + GetNumSinglePageShadowMaps()) * FVirtualShadowMap::MaxMipLevels
	// this avoids clearing unused slots.
	if (Index.x < NumPageRectsToClear)
	{
		uint RectOffset = Index.x;
		// The full shadow maps are offset to a distant part of the ID range
		if (Index.x >= VirtualShadowMap.NumSinglePageShadowMaps * VSM_MAX_MIP_LEVELS )
		{
			RectOffset += VSM_MAX_SINGLE_PAGE_SHADOW_MAPS * VSM_MAX_MIP_LEVELS - VirtualShadowMap.NumSinglePageShadowMaps * VSM_MAX_MIP_LEVELS;
		}
		uint4 Empty = uint4(VSM_LEVEL0_DIM_PAGES_XY, VSM_LEVEL0_DIM_PAGES_XY, 0, 0);
		OutUncachedPageRectBounds[RectOffset] = Empty;
		OutAllocatedPageRectBounds[RectOffset] = Empty;
	}

	// Clear the various list counters
	if (Index.x == 0)
	{
		// Thiese lists are going to start "full" before packing
		SetPhysicalPageListCount(PHYSICAL_PAGE_LIST_LRU, VirtualShadowMap.MaxPhysicalPages);
		// These start empty and are added to as elements are removed from the LRU one
		SetPhysicalPageListCount(PHYSICAL_PAGE_LIST_AVAILABLE, 0);
		SetPhysicalPageListCount(PHYSICAL_PAGE_LIST_EMPTY, 0);
		SetPhysicalPageListCount(PHYSICAL_PAGE_LIST_REQUESTED, 0);
	}
}


// Mapping of previous frame/update data to current frame
StructuredBuffer<FNextVirtualShadowMapData> NextVirtualShadowMapData;
uint NextVirtualShadowMapDataCount;

// To propogate any invalidation flags to the physical page flags
Texture2D<uint> PrevPageRequestFlags;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void UpdatePhysicalPageAddresses(uint3 Index : SV_DispatchThreadID)
{	
	// TODO: Make this a loose constant probably to remove dependency of this shader on VSM UB
	// Still needs the VSM defines for IsVirtualShadowMapPageAddressValid addressing math though!
	if (Index.x >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}

	// Use identity mapping by default
	int PhysicalPageIndex = Index.x;
	checkStructuredBufferAccessSlow(OutPhysicalPageMetaData, PhysicalPageIndex);
	
#if HAS_CACHE_DATA
	FPhysicalPageMetaData PrevMetaData = OutPhysicalPageMetaData[PhysicalPageIndex];

	FVSMPageOffset PrevGlobalPageOffset = FVSMPageOffset::Unpack(0u);
	bool bKeepPage = false;
	if (PrevMetaData.Flags != 0)
	{
		// Update virtual shadow map ID to the equivalent one this frame if present
		// NOTE: We need a range check as we only add elements to this mapping if they exist this frame
		FVirtualShadowMapHandle PrevVirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(PrevMetaData.VirtualShadowMapId);
		if (PrevVirtualShadowMapHandle.IsValid() && PrevVirtualShadowMapHandle.GetDataIndex() < NextVirtualShadowMapDataCount)
		{
			PrevGlobalPageOffset = CalcPageOffset(PrevVirtualShadowMapHandle, PrevMetaData.MipLevel, PrevMetaData.PageAddress);

			FNextVirtualShadowMapData NextData = NextVirtualShadowMapData[PrevVirtualShadowMapHandle.GetDataIndex()];
			
			// Check if it maps to a valid virtual shadow map for which we wish to keep data
			if (NextData.Flags & VSM_NEXT_FLAG_KEEP_PAGE)
			{
				FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(NextData.NextVirtualShadowMapId);
				// Clipmap panning; zeroed otherwise so safe
				int2 TestPageAddress = int2(PrevMetaData.PageAddress) + NextData.PageAddressOffset;
				if (IsVirtualShadowMapPageAddressValid(TestPageAddress, PrevMetaData.MipLevel))
				{
					// Valid physical page in the cache!
					// It may still be invalidated by flags or over-written by new requests this frame, but for now we will maintain it
					OutPhysicalPageMetaData[PhysicalPageIndex].VirtualShadowMapId = VirtualShadowMapHandle.Id;
					OutPhysicalPageMetaData[PhysicalPageIndex].PageAddress = uint2(TestPageAddress);
					// No changes to other fields
					bKeepPage = true;
				}
			}
		}
	}

	if (bKeepPage)
	{
		// Propogate any invalidation flags from the previous page requests to the physical page
		const uint PrevFlags = PrevPageRequestFlags[PrevGlobalPageOffset.GetResourceAddress()];
		const uint InvalidationFlags = PrevFlags & VSM_EXTENDED_FLAG_ANY_INVALIDATED;
		if (InvalidationFlags != 0)
		{
			// Add them to any previous flags
			OutPhysicalPageMetaData[PhysicalPageIndex].Flags = PrevMetaData.Flags | InvalidationFlags;
		}
	}
	else
	{
		// Only need to zero out flags for it to be considered invalid
		OutPhysicalPageMetaData[PhysicalPageIndex].Flags = 0;
	}
#else
	OutPhysicalPageMetaData[PhysicalPageIndex].Flags = 0;
#endif
}

int bDynamicPageInvalidation;
int bAllocateViaLRU;
int MaxPageAgeSinceLastRequest;

// Shadow.Virtual.DirtyPageFlags storage. Allocated with VSM_DIRTY_PAGE_FLAGS_NUM_SLICES * MaxPhysicalPages
// per-page slices plus one trailing element used as the deferred-invalidation budget atomic.
RWStructuredBuffer<uint> DirtyPageFlagsInOut;

// Per-frame budget for the deferred-invalidation extended-flag channel. -1 unbounded, 0 disabled, >0 max per frame.
int DeferredInvalidationBudget;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void UpdatePhysicalPages(uint3 Index : SV_DispatchThreadID)
{	
	// Because of launch size rounding we might get here.
	if (Index.x >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}

	// This is the index in the PhysicalPageList
	const uint PhysicalPageListIndex = Index.x;
	bool bRemovedPageFromList = false;

	// Use identity mapping by default
	int PhysicalPageIndex = PhysicalPageListIndex;

#if HAS_CACHE_DATA
	if (bAllocateViaLRU)
	{
		// If available, use last frame's LRU ordering as the input here so we can maintain that order
		// NOTE: These end up sorted into the PHYSICAL_PAGE_LIST_REQUESTED list at the end of the frame
		// LastFrameLRUList[PhysicalPageListIndex]
		const int PrevPageListStart = GetPhysicalPageListStart(PHYSICAL_PAGE_LIST_REQUESTED);
		PhysicalPageIndex = PrevPhysicalPageLists[PrevPageListStart + PhysicalPageListIndex];
		checkSlow(PhysicalPageIndex >= INDEX_NONE);
		checkSlow(PhysicalPageIndex < VirtualShadowMap.MaxPhysicalPages);
	}
#endif

	checkStructuredBufferAccessSlow(OutPhysicalPageMetaData, PhysicalPageIndex);

	// 1:1 read modify write is safe
	uint NextPhysicalFlags = 0;

#if HAS_CACHE_DATA
	{
		FPhysicalPageMetaData PrevMetaData = OutPhysicalPageMetaData[PhysicalPageIndex];
		uint MipLevel = PrevMetaData.MipLevel;
		
		if (PrevMetaData.Flags != 0)
		{
			// Convenience
			const FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(PrevMetaData.VirtualShadowMapId);
			const uint2 PageAddress = PrevMetaData.PageAddress;

			// Look up the request flags for this frame to see if this page was requested again
			const FVSMPageOffset GlobalPageOffset = CalcPageOffset(VirtualShadowMapHandle, MipLevel, PageAddress);
			const uint RequestFlags = PageRequestFlags[GlobalPageOffset.GetResourceAddress()];
			const bool bRequestedThisFrame = RequestFlags != 0;

			const int PhysicalPageRequestedAge = int(VirtualShadowMap.SceneFrameNumber - PrevMetaData.LastRequestedSceneFrameNumber);

			// If the light is unreferenced we also allow its pages to live (unless reallocated) regardless of age for now
			// since we won't be rendering into them so they don't do a lot of harm being present.
			// TODO: Revisit this... probably make it just age-based now
			const FVirtualShadowMapProjectionShaderData Projection = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
			if (bRequestedThisFrame || Projection.bUnreferenced || PhysicalPageRequestedAge <= MaxPageAgeSinceLastRequest)
			{
				const uint PrevPhysicalFlags = PrevMetaData.Flags;

				// Update the mapping data for any valid cached pages so we don't lose it
				OutPhysicalPageMetaData[PhysicalPageIndex].VirtualShadowMapId = VirtualShadowMapHandle.Id;
				OutPhysicalPageMetaData[PhysicalPageIndex].PageAddress = PageAddress;

				if (!bRequestedThisFrame || Projection.bUnreferenced)
				{
					// If the page is unrefereced (i.e. we are not going to render to it this frame) we want to leave the physical
					// metadata alone, *specifically* the invalidation flags. Since an unreferenced page will not get
					// rendered to this frame, we can't clear these flags and instead want to maintain them until a potential
					// future frame when this page might be referenced again.
					// Tag the page so we can skip it in rendering-related tasks like clearing and merging
					NextPhysicalFlags = PrevPhysicalFlags | VSM_EXTENDED_FLAG_UNREFERENCED;

					// NOTE: This should be unused during this render, but may be used by invalidation between frames/renders
					// We only want to set ALLOCATED so that it gets picked up by invalidation, but not by
					// any rendering this frame. Any invalidation flags already on the physical page remain there,
					// so this is just for new ones generated this frame to ensure we invalidate any cached-but-currently-unused
					// pages still in the pool
					OutPageFlags[GlobalPageOffset.GetResourceAddress()] = VSM_FLAG_ALLOCATED;
				}
				else
				{
					uint NextPageFlags = VSM_FLAG_ALLOCATED;
					const uint RequestAuxFlags = RequestFlags & VSM_FLAG_AUX_MASK;
					const uint RequestDetailGeometryFlag = (RequestFlags & VSM_FLAG_DETAIL_GEOMETRY);

					bool bIsStaticInvalidated = (PrevPhysicalFlags & VSM_EXTENDED_FLAG_INVALIDATE_STATIC) != 0u;

					bool bSkipDynamicInvalidations = VirtualShadowMapHandle.IsSinglePage()
						|| (RequestDetailGeometryFlag == 0u && Projection.bForceCacheDynamicCoarse && !bIsStaticInvalidated);

					bool bKeepDeferredInvalidationFlag = false;
					// Distant lights ignore invalidations as they are round-robin invalidated
					if (bDynamicPageInvalidation && !bSkipDynamicInvalidations)
					{
						if ((PrevPhysicalFlags & VSM_EXTENDED_FLAG_ANY_INVALIDATED) != 0)
						{
							if ((PrevPhysicalFlags & VSM_EXTENDED_FLAG_INVALIDATE_STATIC) == 0)
							{
								// ONLY dynamic is invalidated, static can remain cached
								NextPageFlags |= VSM_FLAG_DYNAMIC_UNCACHED;
							}
							else
							{
								// Invalidate both
								NextPageFlags |= VSM_FLAG_ANY_UNCACHED;
							}
						}

						if ((PrevPhysicalFlags & VSM_EXTENDED_FLAG_DEFERRED_INVALIDATION) != 0u)
						{
							bool bWithinBudget;
							if (DeferredInvalidationBudget < 0)
							{
								bWithinBudget = true;
							}
							else if (DeferredInvalidationBudget == 0)
							{
								bWithinBudget = false;
							}
							else
							{
								// Atomic counter shares the tail of DirtyPageFlagsInOut, at index MaxPhysicalPages * VSM_DIRTY_PAGE_FLAGS_NUM_SLICES.
								uint Slot;
								WaveInterlockedAddScalar_(DirtyPageFlagsInOut[VirtualShadowMap.MaxPhysicalPages * VSM_DIRTY_PAGE_FLAGS_NUM_SLICES], 1u, Slot);
								bWithinBudget = (Slot < (uint)DeferredInvalidationBudget);
							}

							if (bWithinBudget)
							{
								NextPageFlags |= VSM_FLAG_ANY_UNCACHED;
							}
							else
							{
								bKeepDeferredInvalidationFlag = true;
							}
						}

						// Always invalidate dynamic when using receiver mask, as the page may be incomplete
						if (Projection.bUseReceiverMask)
						{
							NextPageFlags |= VSM_FLAG_DYNAMIC_UNCACHED;
						}
					}
					else
					{
						bKeepDeferredInvalidationFlag = (PrevPhysicalFlags & VSM_EXTENDED_FLAG_DEFERRED_INVALIDATION) != 0u;
					}

					uint PhysicalPageAuxFlags = (PrevPhysicalFlags & VSM_FLAG_AUX_MASK);
					if (bRequestedThisFrame)
					{
						StatsBufferInterlockedInc(VSM_STAT_REQUESTED_THIS_FRAME_PAGES);
						// Remove from LRU list and add to requested list
						PushPhysicalPageList(PHYSICAL_PAGE_LIST_REQUESTED, PhysicalPageIndex);
						OutPhysicalPageMetaData[PhysicalPageIndex].LastRequestedSceneFrameNumber = VirtualShadowMap.SceneFrameNumber;
						bRemovedPageFromList = true;

						// If the detail geometry flag doesn't match the cached page we treat it as a full invalidation.
						// TODO: This could potentially be a problem for interleaved multiview rendering;
						// If the flag differs in the two views it will cause cache thrashing.			
						// Note: also not really needed for VSM_FLAG_PRIMARY_REQUEST, as that is dynamic anyway						
						if (RequestAuxFlags != PhysicalPageAuxFlags)
						{
							NextPageFlags |= (VSM_FLAG_STATIC_UNCACHED | VSM_FLAG_DYNAMIC_UNCACHED);
							PhysicalPageAuxFlags = RequestAuxFlags;
						}

						// Only increment the stats for pages requested this render, otherwise it gets confusing
						if (NextPageFlags & VSM_FLAG_STATIC_UNCACHED)
						{
							StatsBufferInterlockedInc(VSM_STAT_STATIC_INVALIDATED_PAGES);
						}
						else
						{
							StatsBufferInterlockedInc(VSM_STAT_STATIC_CACHED_PAGES);
						}
						if (NextPageFlags & VSM_FLAG_DYNAMIC_UNCACHED)
						{
							StatsBufferInterlockedInc(VSM_STAT_DYNAMIC_INVALIDATED_PAGES);
						}
						else
						{
							StatsBufferInterlockedInc(VSM_STAT_DYNAMIC_CACHED_PAGES);
						}
					}
					NextPageFlags |= PhysicalPageAuxFlags;
							
					const uint PhysicalFlags = (Projection.bUnCached ? VSM_EXTENDED_FLAG_VIEW_UNCACHED : 0U);
					NextPhysicalFlags = NextPageFlags | PhysicalFlags;

					// Maintain any previous initialized flags for this page
					NextPhysicalFlags |= (PrevPhysicalFlags & VSM_EXTENDED_FLAG_ANY_INITIALIZED);

					if (bKeepDeferredInvalidationFlag)
					{
						NextPhysicalFlags |= VSM_EXTENDED_FLAG_DEFERRED_INVALIDATION;
					}

					// If the page is going to be fully cached, but the VSM_EXTENDED_FLAG_FORCE_CACHED flag is on,
					// we want to do something special here. We want to consider this page for any rendering in case
					// WPO distance disable has changed (which we want to start causing invalidations), but not actually
					// invalidate or render anything into it. Currently we accomplish this by setitng the DYNAMIC_UNCACHED
					// flag in the hierarchical page flags (WPO only ever gets rendered into dynamic cache -
					// see ShouldCacheInstanceAsStatic), but NOT on the physical page (which would indicate a real invalidation).
					// This can get significantly cleaned up if/when we free up an additional hierarchical page flag bit.
					// We do NOT want these details showing up in debug visualizations or cache stats though
					bool bPageValidForRendering = (NextPageFlags & VSM_FLAG_ANY_UNCACHED) != 0;
					if (!bSkipDynamicInvalidations && PrevPhysicalFlags & VSM_EXTENDED_FLAG_FORCE_CACHED)
					{
						// NOTE: WPO can only ever be dynamic cached
						NextPageFlags |= VSM_FLAG_DYNAMIC_UNCACHED;
						StatsBufferInterlockedInc(VSM_STAT_WPO_CONSIDERED_PAGES);
					}

					// Map the page to the physical page
					// If we later allocate over top of this page (for one requested this frame), we will zero this out again. See AllocateNewPageMappings
					OutPageTable[GlobalPageOffset.GetResourceAddress()] = ShadowEncodePageTable(VSMPhysicalIndexToPageAddress(PhysicalPageIndex), bPageValidForRendering, false);
					OutPageFlags[GlobalPageOffset.GetResourceAddress()] = NextPageFlags;
				} // Unreferenced
			}
		}
	}
#endif

	// If page is invalidated/empty, remove it from the LRU list and add it to the empty list
	// It will be re-added after packing to the end of the AVAILABLE list
	if (NextPhysicalFlags == 0)
	{
		StatsBufferInterlockedInc(VSM_STAT_EMPTY_PAGES);
		PushPhysicalPageList(PHYSICAL_PAGE_LIST_EMPTY, PhysicalPageIndex);
		bRemovedPageFromList = true;
	}

	OutPhysicalPageMetaData[PhysicalPageIndex].Flags = NextPhysicalFlags;

	// Write out the LRU list while maintaining order, with anything we removed marked as INDEX_NONE
	SetPhysicalPageListItem(PHYSICAL_PAGE_LIST_LRU, PhysicalPageListIndex, bRemovedPageFromList ? INDEX_NONE : PhysicalPageIndex);
}

void AllocateNewPageMappings(FVirtualShadowMapHandle VirtualShadowMapHandle, FVSMPageOffset GlobalPageOffset, uint MipLevel, uint2 PageAddress)
{
	const uint RequestFlags = PageRequestFlags[GlobalPageOffset.GetResourceAddress()];
	if (RequestFlags != 0)
	{
		// See if we already hooked this up to a mapped page
		const uint PageFlags = (OutPageFlags[GlobalPageOffset.GetResourceAddress()] & VSM_PAGE_FLAGS_BITS_MASK);

		if (PageFlags == 0u)
		{
			StatsBufferInterlockedInc(VSM_STAT_REQUESTED_THIS_FRAME_PAGES);

			int PhysicalPageIndex = PopPhysicalPageList(PHYSICAL_PAGE_LIST_AVAILABLE);
			if (PhysicalPageIndex >= 0)
			{
				StatsBufferInterlockedInc(VSM_STAT_ALLOCATED_NEW);

				// Add back to the end of the requested list
				PushPhysicalPageList(PHYSICAL_PAGE_LIST_REQUESTED, PhysicalPageIndex);

				uint2 PhysicalPageAddress = VSMPhysicalIndexToPageAddress(PhysicalPageIndex);

				// FIRST, check if there's a valid page already mapped to this physical page
				// If so, we must go back and clear out its page table entry before we reallocate this page
				uint PrevPhysicalFlags = 0;
				{
					FPhysicalPageMetaData PrevMetaData = OutPhysicalPageMetaData[PhysicalPageIndex];
					PrevPhysicalFlags = PrevMetaData.Flags;
					if (PrevPhysicalFlags & VSM_FLAG_ALLOCATED)
					{
						FVSMPageOffset PrevGlobalPageOffset = CalcPageOffset(FVirtualShadowMapHandle::MakeFromId(PrevMetaData.VirtualShadowMapId), PrevMetaData.MipLevel, PrevMetaData.PageAddress);
						OutPageTable[PrevGlobalPageOffset.GetResourceAddress()] = 0;
						OutPageFlags[PrevGlobalPageOffset.GetResourceAddress()] = 0;
					}
				}

				uint RequestAuxFlags = RequestFlags & VSM_FLAG_AUX_MASK;
				uint Flags = VSM_FLAG_ALLOCATED | VSM_FLAG_DYNAMIC_UNCACHED | VSM_FLAG_STATIC_UNCACHED | RequestAuxFlags;

				// Mark this page as allocated and not cached (always valid for rendering)
				OutPageTable[GlobalPageOffset.GetResourceAddress()] = ShadowEncodePageTable(PhysicalPageAddress, true, (Flags & VSM_FLAG_SECONDARY_REQUEST) != 0u);
				OutPageFlags[GlobalPageOffset.GetResourceAddress()] = Flags;

				const FVirtualShadowMapProjectionShaderData Projection = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
				uint PhysicalFlags = (Projection.bUnCached ? VSM_EXTENDED_FLAG_VIEW_UNCACHED : 0U);

				// NOTE: We cannot maintain the "initialized" flags in this path because initialized for dynamic pages
				// can mean it has a copy of the static page data, which is not valid for a reused page.
				// There may be some edge cases where we can avoid reclearing empty static pages, but they are rare.

				OutPhysicalPageMetaData[PhysicalPageIndex].Flags = Flags | PhysicalFlags;
				OutPhysicalPageMetaData[PhysicalPageIndex].LastRequestedSceneFrameNumber = VirtualShadowMap.SceneFrameNumber;
				OutPhysicalPageMetaData[PhysicalPageIndex].VirtualShadowMapId = VirtualShadowMapHandle.Id;
				OutPhysicalPageMetaData[PhysicalPageIndex].MipLevel = MipLevel;
				OutPhysicalPageMetaData[PhysicalPageIndex].PageAddress = PageAddress;
			}
			else
			{
				// We end up here if we're out of physical pages, this means some parts get no physical backing provided.
				// Post this error condition back to the host somehow!
				// Probably want to know if we're getting close even.
				//OutPageTable[GlobalPageOffset] = 0;
				//OutPageFlags[GlobalPageOffset] = 0;
			}
		}
	}
}

#ifdef ClearPageTableCS

RWTexture2D<uint> OutDestBuffer;
uint ClearValue;
uint SampleStride;

#define HAS_MIP_LEVELS (NUM_MIP_LEVELS > 1)

#if HAS_MIP_LEVELS
RWTexture2D<uint> OutDestBufferMips_0;
RWTexture2D<uint> OutDestBufferMips_1;
RWTexture2D<uint> OutDestBufferMips_2;
RWTexture2D<uint> OutDestBufferMips_3;
RWTexture2D<uint> OutDestBufferMips_4;
RWTexture2D<uint> OutDestBufferMips_5;
RWTexture2D<uint> OutDestBufferMips_6;

void ClearMip(RWTexture2D<uint> OutDestBufferMip, uint HMipLevel, FVirtualSMLevelOffset LevelOffset, uint2 PageCoord, uint SampleStrideLocal, uint MipLevel)
{
	uint LevelDim = CalcLevelDimsPages(MipLevel + HMipLevel) * SampleStrideLocal;
	uint2 HMipOffset = (LevelOffset.LevelTexelOffset * SampleStrideLocal) >> HMipLevel;
	if (all(PageCoord < LevelDim))
	{
		OutDestBufferMip[HMipOffset + PageCoord] = ClearValue;
	}
}

#endif

struct FClearPageTableWorker
{
	void Run(FPerPageDispatchSetup Setup)
	{
		for (uint MipLevel = Setup.MipLevelStart; MipLevel < Setup.MipLevelEnd; ++MipLevel)
		{
			FVirtualSMLevelOffset LevelOffset = CalcPageTableLevelOffset(Setup.VirtualShadowMapHandle, MipLevel);

			uint LoopEndXY = Setup.GetLoopEnd(MipLevel);
			for (uint PageY = Setup.LoopStart.y; PageY < LoopEndXY; PageY += Setup.LoopStride)
			{
				for (uint PageX = Setup.LoopStart.x; PageX < LoopEndXY; PageX += Setup.LoopStride)
				{
					const FVSMPageOffset PageOffset = CalcPageOffset(LevelOffset, MipLevel, uint2(PageX, PageY));
					BRANCH
					if (SampleStride == 2u)
					{
						OutDestBuffer[PageOffset.GetResourceAddress() * 2u + uint2(0u, 0u)] = ClearValue;
						OutDestBuffer[PageOffset.GetResourceAddress() * 2u + uint2(1u, 0u)] = ClearValue;
						OutDestBuffer[PageOffset.GetResourceAddress() * 2u + uint2(0u, 1u)] = ClearValue;
						OutDestBuffer[PageOffset.GetResourceAddress() * 2u + uint2(1u, 1u)] = ClearValue;
#if HAS_MIP_LEVELS

#define DO_MIP_LEVEL(_HLevel_) ClearMip(OutDestBufferMips_##_HLevel_,_HLevel_ + 1, LevelOffset, uint2(PageX, PageY), 2u, MipLevel)
					DO_MIP_LEVEL(0);
					DO_MIP_LEVEL(1);
					DO_MIP_LEVEL(2);
					DO_MIP_LEVEL(3);
					DO_MIP_LEVEL(4);
					DO_MIP_LEVEL(5);
					// Note: the NUM_MIP_LEVELS includes the base level, which is not included in the UAV array
#if NUM_MIP_LEVELS > 7
					DO_MIP_LEVEL(6);
#endif
#undef DO_MIP_LEVEL

#endif
					}
					else
					{
						OutDestBuffer[PageOffset.GetResourceAddress()] = ClearValue;

#if HAS_MIP_LEVELS
#define DO_MIP_LEVEL(_HLevel_) ClearMip(OutDestBufferMips_##_HLevel_,_HLevel_ + 1, LevelOffset, uint2(PageX, PageY), 1u, MipLevel)
					DO_MIP_LEVEL(0);
					DO_MIP_LEVEL(1);
					DO_MIP_LEVEL(2);
					DO_MIP_LEVEL(3);
					DO_MIP_LEVEL(4);
					DO_MIP_LEVEL(5);
					// Note: the NUM_MIP_LEVELS includes the base level, which is not included in the UAV array
#if NUM_MIP_LEVELS > 7
					DO_MIP_LEVEL(6);
#endif
#undef DO_MIP_LEVEL
#endif
					}
				}
			}
		}
	}
};

/**
 */
[numthreads(PER_PAGE_THREAD_GROUP_SIZE_XY, PER_PAGE_THREAD_GROUP_SIZE_XY, 1)]
void ClearPageTableCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FClearPageTableWorker ClearPageTableWorker;
	FPerPageDispatchSetup Setup; 
	Setup.Execute(DispatchThreadId, ClearPageTableWorker);
}

#endif // ClearPageTableCS

#ifdef AllocateNewPageMappingsCS

struct FAllocateNewPageMappingsWorker
{
	void Run(FPerPageDispatchSetup Setup)
	{
		for (uint MipLevel = Setup.MipLevelStart; MipLevel < Setup.MipLevelEnd; ++MipLevel)
		{
			uint LoopEndXY = Setup.GetLoopEnd(MipLevel);
			for (uint PageY = Setup.LoopStart.y; PageY < LoopEndXY; PageY += Setup.LoopStride)
			{
				for (uint PageX = Setup.LoopStart.x; PageX < LoopEndXY; PageX += Setup.LoopStride)
				{
					const FVSMPageOffset PageOffset = CalcPageOffset(Setup.VirtualShadowMapHandle, MipLevel, uint2(PageX, PageY));
					AllocateNewPageMappings(Setup.VirtualShadowMapHandle, PageOffset, MipLevel, uint2(PageX, PageY));
				}
			}
		}
	}
};

/**
 */
[numthreads(PER_PAGE_THREAD_GROUP_SIZE_XY, PER_PAGE_THREAD_GROUP_SIZE_XY, 1)]
void AllocateNewPageMappingsCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	FAllocateNewPageMappingsWorker AllocateNewPageMappingsWorker;
	FPerPageDispatchSetup Setup; 
	Setup.Execute(DispatchThreadId, AllocateNewPageMappingsWorker);
}

#endif // AllocateNewPageMappingsCS

// NOTE: We only launch a single group here for now to avoid multi-pass so we really want it as large as possible
// Can optimize this later if needed for larger physical page counts
#define NUM_THREADS_PER_GROUP 1024
#include "../ThreadGroupPrefixSum.ush"

[numthreads(NUM_THREADS_PER_GROUP, 1, 1)]
void PackAvailablePages(uint GroupIndex : SV_GroupIndex)
{
	int TotalCount = 0;

	// Must be a uniform loop
	for (int GroupStart = 0; GroupStart < VirtualShadowMap.MaxPhysicalPages; GroupStart += NUM_THREADS_PER_GROUP)
	{
		int ListIndex = GroupStart + GroupIndex;

		int PhysicalPageIndex = ListIndex < VirtualShadowMap.MaxPhysicalPages ? 
			GetPhysicalPageListItem(PHYSICAL_PAGE_LIST_LRU, ListIndex) :
			INDEX_NONE;

		bool bListItemValid = PhysicalPageIndex != INDEX_NONE;

		int SumValue = bListItemValid ? 1 : 0;
		// NOTE: Cannot be under any divergent branching!
		int GroupCount = 0;
		int Offset = ThreadGroupPrefixSum(SumValue, GroupIndex, GroupCount);

		if (bListItemValid)
		{
			SetPhysicalPageListItem(PHYSICAL_PAGE_LIST_AVAILABLE, TotalCount + Offset, PhysicalPageIndex);
		}
		TotalCount += GroupCount;

		// This should already be accounted for internally by ThreadGroupPrefixSum, but putting one here
		// to be absolutely sure.
		GroupMemoryBarrierWithGroupSync();
	}

	// Set total number
	if (GroupIndex == 0)
	{
		SetPhysicalPageListCount(PHYSICAL_PAGE_LIST_AVAILABLE, TotalCount);
	}
}

#undef NUM_THREADS_PER_GROUP

uint bAppendEmptyToAvailable;
// If true, simply updates the counts instead of copying items
// This should be run with the same parameters right after the copy pass, with a single group
uint bUpdateCounts;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void AppendPhysicalPageLists(uint ThreadId : SV_DispatchThreadID)
{
	// We only need two variants currently, EMPTY->AVAILABLE and AVAILABLE->REQUESTED
	int InputList  = bAppendEmptyToAvailable ? PHYSICAL_PAGE_LIST_EMPTY : PHYSICAL_PAGE_LIST_AVAILABLE;
	int OutputList = bAppendEmptyToAvailable ? PHYSICAL_PAGE_LIST_AVAILABLE : PHYSICAL_PAGE_LIST_REQUESTED;

	// NOTE: This needs to maintain order!
	// It also needs to be robust against physical page pool overflows, ensuring that we never "lose" any
	// items in the final LRU list for the next frame.
	int InputCount = GetPhysicalPageListCount(InputList);
	int OutputCount = GetPhysicalPageListCount(OutputList);
	int CopyCount = max(0, min(InputCount, int(VirtualShadowMap.MaxPhysicalPages) - OutputCount));

	if (bUpdateCounts)
	{
		// Update pass (after copy pass)
		if (ThreadId == 0)
		{
			int NewOutputCount = OutputCount + CopyCount;
			SetPhysicalPageListCount(OutputList, NewOutputCount);
			SetPhysicalPageListCount(InputList, 0);

			// The REQUESTED list needs to specifically end up with a single unique copy of each index as
			// this becomes the LRU list for the next update. If we were to lose any indices or list entries
			// then we would also (permanently) lose actual physical pages. Thus we assert that at least the
			// list must end up as the right size after the final append!
			if (!bAppendEmptyToAvailable)
			{
				/*
				PLATFORM_ASSERT4(
					NewOutputCount == VirtualShadowMap.MaxPhysicalPages,
					0xCECC,
					__LINE__,
					InputCount,
					OutputCount,
					NewOutputCount);
				*/
				checkSlow(NewOutputCount == VirtualShadowMap.MaxPhysicalPages);
			}
			else
			{
				/*
				// All pages should now be in PHYSICAL_PAGE_LIS_AVAILABLE or REQUESTED
				int AvailableCount = GetPhysicalPageListCount(PHYSICAL_PAGE_LIST_AVAILABLE);
				int RequestedCount = GetPhysicalPageListCount(PHYSICAL_PAGE_LIST_REQUESTED);
				int EmptyCount = GetPhysicalPageListCount(PHYSICAL_PAGE_LIST_EMPTY);
				int TotalPages = AvailableCount + RequestedCount;

				PLATFORM_ASSERT4(
					TotalPages == VirtualShadowMap.MaxPhysicalPages,
					0xCECC,
					__LINE__,
					AvailableCount,
					RequestedCount,
					EmptyCount);
				*/
			}
		}
	}
	else
	{
		if (ThreadId < CopyCount)
		{
			int InputItem = GetPhysicalPageListItem(InputList, ThreadId);
			SetPhysicalPageListItem(OutputList, OutputCount + ThreadId, InputItem);
		}
	}
}

StructuredBuffer<FPhysicalPageMetaData> PhysicalPageMetaData;
RWTexture2DArray<uint> OutPhysicalPagePool;

// Helper function to merge static and dynamic depth.
void MergePhysicalPixel(uint2 PixelCoord)
{
	// 1:1 pixels so this is safe RMW
	OutPhysicalPagePool[uint3(PixelCoord, 0)] = max(
		OutPhysicalPagePool[uint3(PixelCoord, 0)],
		OutPhysicalPagePool[uint3(PixelCoord, GetVirtualShadowMapStaticArrayIndex())]);
}

// Log2 2D dimension of thread group size, 2^4 == 16, 
#define LOG2_TILE_THREAD_GROUP_SIZE_XY 4u
#define TILE_THREAD_GROUP_SIZE_XY (1u << LOG2_TILE_THREAD_GROUP_SIZE_XY)

// Each thread takes 2x2 samples to work with, so tile size is 2x thread group size
#define LOG2_TILE_SIZE_XY (LOG2_TILE_THREAD_GROUP_SIZE_XY + 1u)

#if VSM_LOG2_PAGE_SIZE < LOG2_TILE_SIZE_XY
#error "VSM_LOG2_PAGE_SIZE must be larger than LOG2_TILE_SIZE, either increase one or reduce the other"
#endif 

// Number of tiles (thread groups) in each dimension to cover the page
#define LOG2_TILES_PER_PAGE_XY ( VSM_LOG2_PAGE_SIZE - LOG2_TILE_SIZE_XY )
// Log2 1D tile count to cover the page  LOG2_TILES_PER_PAGE_XY * LOG2_TILES_PER_PAGE_XY
#define LOG2_TILES_PER_PAGE_1D ( 2U * LOG2_TILES_PER_PAGE_XY )
// 1D tile count to cover the page  
#define TILES_PER_PAGE_1D ( 1U << LOG2_TILES_PER_PAGE_1D )

#define TILES_PER_PAGE_XY_MASK ( ( 1U << LOG2_TILES_PER_PAGE_XY ) - 1U )
#define TILES_PER_PAGE_1D_MASK ( ( 1U << LOG2_TILES_PER_PAGE_1D ) - 1U )

RWBuffer<uint> OutInitializePagesIndirectArgsBuffer;
RWStructuredBuffer<uint> OutPhysicalPagesToInitialize;
int bUseClearedFlags;

void EmitPageToProcess(RWBuffer<uint> OutIndirectArgsBuffer, uint ArgsBufferOffset, RWStructuredBuffer<uint> OutSelectedPhysicalIndexBuffer, uint PhysicalPageIndex)
{
	int GroupCount = 0;
	// Each page needs TILES_PER_PAGE_1D groups launched
	WaveInterlockedAddScalar_(OutIndirectArgsBuffer[ArgsBufferOffset], TILES_PER_PAGE_1D, GroupCount);
	OutSelectedPhysicalIndexBuffer[GroupCount >> LOG2_TILES_PER_PAGE_1D] = PhysicalPageIndex;
}

void EmitPageToProcess(RWBuffer<uint> OutIndirectArgsBuffer, RWStructuredBuffer<uint> OutSelectedPhysicalIndexBuffer, uint PhysicalPageIndex)
{
	EmitPageToProcess(OutIndirectArgsBuffer, 0u, OutSelectedPhysicalIndexBuffer, PhysicalPageIndex);
}

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void SelectPagesToInitializeCS(uint PhysicalPageIndex : SV_DispatchThreadID)
{
	if (PhysicalPageIndex >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}

	FPhysicalPageMetaData MetaData = OutPhysicalPageMetaData[PhysicalPageIndex];

	bool bUnreferenced = (MetaData.Flags & VSM_EXTENDED_FLAG_UNREFERENCED) != 0;
	bool bFullyCached = (MetaData.Flags & VSM_FLAG_ANY_UNCACHED) == 0;
	bool bForceCached = (MetaData.Flags & VSM_EXTENDED_FLAG_FORCE_CACHED) != 0;

	if ((MetaData.Flags & VSM_FLAG_ALLOCATED) == 0)
	{
		// Page not used, we're done
	}
	else if (bUnreferenced || bFullyCached || bForceCached)
	{
		// Page fully cached or unreferenced. Leave the data alone.
	}
	else
	{
		// At least one of the pages is uncached
		uint ClearedFlags = 0;

		// Clear static page if needed and it is not already cleared
		if ((MetaData.Flags & VSM_FLAG_STATIC_UNCACHED) &&
			(MetaData.Flags & VSM_EXTENDED_FLAG_VIEW_UNCACHED) == 0U)
		{
			if (bUseClearedFlags == 0 || (MetaData.Flags & VSM_EXTENDED_FLAG_STATIC_INITIALIZED) == 0)
			{
				EmitPageToProcess(OutInitializePagesIndirectArgsBuffer, OutPhysicalPagesToInitialize, PhysicalPageIndex + VirtualShadowMap.MaxPhysicalPages);
				StatsBufferInterlockedInc(VSM_STAT_NUM_PAGES_TO_CLEAR);
				ClearedFlags |= VSM_EXTENDED_FLAG_STATIC_INITIALIZED | VSM_EXTENDED_FLAG_STATIC_DIRTY;
			}
		}

		// Clear dynamic page if it is not already cleared
		// Also clear it in all cases if we're clearing the static page, as that means our "initialized" data
		// is also now out of date.
		if (bUseClearedFlags == 0 ||
			(MetaData.Flags & VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED) == 0 ||
			ClearedFlags != 0)
		{
			EmitPageToProcess(OutInitializePagesIndirectArgsBuffer, OutPhysicalPagesToInitialize, PhysicalPageIndex);
			StatsBufferInterlockedInc(VSM_STAT_NUM_PAGES_TO_CLEAR);
			// NOTE: We also set DIRTY as we want to trigger anything related to this page changing,
			// i.e. HZB calculation, merging as appropriate.
			ClearedFlags |= VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED | VSM_EXTENDED_FLAG_DYNAMIC_DIRTY;
		}

		// Set cleared flags
		OutPhysicalPageMetaData[PhysicalPageIndex].Flags = (MetaData.Flags | ClearedFlags);
	}
}

struct FTileInputSetup
{
	uint2 TileOffsetInPage;
	uint PageInputIndex;
};

FTileInputSetup GetTileInputSetup(uint GroupIndex)
{
	FTileInputSetup Result;
	Result.PageInputIndex = GroupIndex >> LOG2_TILES_PER_PAGE_1D;
	// Each page has 1 << LOG2_TILES_PER_PAGE_XY groups (aka tiles) assigned to work on it.
	const uint LocalTileIndex = GroupIndex & TILES_PER_PAGE_1D_MASK;
	// wrap to 2D tile coord & figure out top-left corner for this workgroup
	Result.TileOffsetInPage = uint2(LocalTileIndex & TILES_PER_PAGE_XY_MASK, LocalTileIndex >> LOG2_TILES_PER_PAGE_XY) << LOG2_TILE_SIZE_XY;
	return Result;
}

struct FTileSetup
{
	FPhysicalPageMetaData MetaData;
	uint PageIndex;
	uint ArrayIndex;
	
	uint2 PhysPageAddress;
	
	uint2 TileOffsetInPage;
};

FTileSetup GetTileSetupBase(uint GroupIndex, StructuredBuffer<uint> PageIndexBuffer)
{
	FTileInputSetup TileInputSetup = GetTileInputSetup(GroupIndex);
	FTileSetup Result;
	Result.TileOffsetInPage = TileInputSetup.TileOffsetInPage;

	Result.PageIndex = PageIndexBuffer[TileInputSetup.PageInputIndex];
	Result.ArrayIndex = 0u;

	if (Result.PageIndex >= VirtualShadowMap.MaxPhysicalPages)
	{
		// Request to clear the static page
		Result.PageIndex -= VirtualShadowMap.MaxPhysicalPages;
		Result.ArrayIndex = 1u;
	}
	Result.PhysPageAddress = VSMPhysicalIndexToPageAddress(Result.PageIndex);

	return Result;
}

FTileSetup GetTileSetup(uint GroupIndex, StructuredBuffer<uint> PageIndexBuffer)
{
	FTileSetup Result = GetTileSetupBase(GroupIndex, PageIndexBuffer);
	Result.MetaData = PhysicalPageMetaData[Result.PageIndex];
	return Result;
}

uint3 GetTileOffset(uint GroupIndex, StructuredBuffer<uint> PageIndexBuffer, inout FPhysicalPageMetaData OutMetaData)
{
	FTileSetup TileSetup = GetTileSetup(GroupIndex, PageIndexBuffer);
	OutMetaData = TileSetup.MetaData;

	// Pixel address of tile region for this thread group.
	const uint2 TileOffset = (TileSetup.PhysPageAddress << VSM_LOG2_PAGE_SIZE) + TileSetup.TileOffsetInPage;
	return uint3(TileOffset, TileSetup.ArrayIndex);
}

uint3 GetTileBasePos(uint2 TileThreadID, uint GroupIndex, StructuredBuffer<uint> PageIndexBuffer, inout FPhysicalPageMetaData OutMetaData)
{
	// Pixel address of tile region for this thread group.
	const uint3 TileOffset = GetTileOffset(GroupIndex, PageIndexBuffer, OutMetaData);
	// Pixel address of 2x2 region to sample for this thread.
	const uint2 BasePos = TileOffset.xy + (TileThreadID.xy << 1u);

	return uint3(BasePos, TileOffset.z);
}

uint3 GetTileBasePos(uint2 TileThreadID, uint GroupIndex, StructuredBuffer<uint> PageIndexBuffer)
{
	FPhysicalPageMetaData TmpMetaData;
	return GetTileBasePos(TileThreadID, GroupIndex, PageIndexBuffer, TmpMetaData);
}

StructuredBuffer<uint> PhysicalPagesToInitialize;

[numthreads(TILE_THREAD_GROUP_SIZE_XY, TILE_THREAD_GROUP_SIZE_XY, 1)]
void InitializePhysicalPagesIndirectCS(uint2 TileThreadID : SV_GroupThreadID, uint GroupIndex : SV_GroupID)
{
	FPhysicalPageMetaData MetaData;
	const uint3 BasePos = GetTileOffset(GroupIndex, PhysicalPagesToInitialize, MetaData) + uint3(TileThreadID, 0);
	const bool bStaticCached = (MetaData.Flags & VSM_FLAG_STATIC_UNCACHED) == 0U;
	const uint Stride = TILE_THREAD_GROUP_SIZE_XY;

	if (bStaticCached && (MetaData.Flags & VSM_EXTENDED_FLAG_VIEW_UNCACHED) == 0U)
	{
		// Initialize from the static page data
		checkSlow(BasePos.z == 0U);
		OutPhysicalPagePool[BasePos + uint3(0U, 0U, 0U)] = OutPhysicalPagePool[BasePos + uint3(0U, 0U, 1U)];
		OutPhysicalPagePool[BasePos + uint3(Stride, 0U, 0U)] = OutPhysicalPagePool[BasePos + uint3(Stride, 0U, 1U)];
		OutPhysicalPagePool[BasePos + uint3(0U, Stride, 0U)] = OutPhysicalPagePool[BasePos + uint3(0U, Stride, 1U)];
		OutPhysicalPagePool[BasePos + uint3(Stride, Stride, 0U)] = OutPhysicalPagePool[BasePos + uint3(Stride, Stride, 1U)];
	}
	else
	{
		// Clear the page to zero
		OutPhysicalPagePool[BasePos + uint3(0U, 0U, 0U)] = 0U;
		OutPhysicalPagePool[BasePos + uint3(Stride, 0U, 0U)] = 0U;
		OutPhysicalPagePool[BasePos + uint3(0U, Stride, 0U)] = 0U;
		OutPhysicalPagePool[BasePos + uint3(Stride, Stride, 0U)] = 0U;
	}
}

#ifdef SelectPagesToPostProcessCS

RWBuffer<uint> OutPostProcessPagesIndirectArgsBuffer;
RWStructuredBuffer<uint> OutPhysicalPagesToMerge;
RWStructuredBuffer<uint> OutPhysicalPagesWithFilterable;
RWStructuredBuffer<uint> OutPhysicalPagesWithSeparableFilterable;
uint PostProcessPagesIndirectArgsBufferStride;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void SelectPagesToPostProcessCS(uint PhysicalPageIndex : SV_DispatchThreadID)
{
	if (PhysicalPageIndex >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}

	FPhysicalPageMetaData MetaData = OutPhysicalPageMetaData[PhysicalPageIndex];

	if ((MetaData.Flags & VSM_FLAG_ALLOCATED) != 0U 
		&& (MetaData.Flags & VSM_EXTENDED_FLAG_UNREFERENCED) == 0U
		&& (MetaData.Flags & VSM_FLAG_SECONDARY_REQUEST) != 0u)
	{
		const FVirtualShadowMapProjectionShaderData Projection = GetVirtualShadowMapProjectionData(FVirtualShadowMapHandle::MakeFromId(MetaData.VirtualShadowMapId));
		if (Projection.bHasPrefilteredDistant)
		{
			{
				int GroupCount = 0;
				// For separable filter, each page needs VSM_PAGE_SIZE (one per row/col) thread groups
				WaveInterlockedAddScalar_(OutPostProcessPagesIndirectArgsBuffer[PostProcessPagesIndirectArgsBufferStride * 1], VSM_PAGE_SIZE, GroupCount);
				OutPhysicalPagesWithSeparableFilterable[GroupCount / VSM_PAGE_SIZE] = PhysicalPageIndex;

				// To process the borders, each page also needs one thread group (technically could derive from above, but would be more serial work
				WaveInterlockedAddScalar(OutPostProcessPagesIndirectArgsBuffer[PostProcessPagesIndirectArgsBufferStride * 3], 1);
			}

			// Emit for merge down pass in 2d tiles (note 3rd args)
			EmitPageToProcess(OutPostProcessPagesIndirectArgsBuffer, PostProcessPagesIndirectArgsBufferStride * 2, OutPhysicalPagesWithFilterable, PhysicalPageIndex);

			StatsBufferInterlockedInc(VSM_STAT_PREFILTERED_DISTANT_PAGES);
		}
	}

	// Merging is only needed if we wrote into the static page
	if ((MetaData.Flags & VSM_FLAG_ALLOCATED) != 0U &&
		(MetaData.Flags & VSM_EXTENDED_FLAG_UNREFERENCED) == 0U &&
		(MetaData.Flags & VSM_EXTENDED_FLAG_VIEW_UNCACHED) == 0U &&
		(MetaData.Flags & VSM_EXTENDED_FLAG_STATIC_DIRTY) != 0U)
	{
		StatsBufferInterlockedInc(VSM_STAT_NUM_PAGES_TO_MERGE);
		EmitPageToProcess(OutPostProcessPagesIndirectArgsBuffer, OutPhysicalPagesToMerge, PhysicalPageIndex);
	}
}

#endif // SelectPagesToPostProcessCS

StructuredBuffer<uint> PhysicalPagesToMerge;

[numthreads(TILE_THREAD_GROUP_SIZE_XY, TILE_THREAD_GROUP_SIZE_XY, 1)]
void MergeStaticPhysicalPagesIndirectCS(uint2 TileThreadID : SV_GroupThreadID, uint GroupIndex : SV_GroupID)
{
	uint2 BasePos = GetTileBasePos(TileThreadID, GroupIndex, PhysicalPagesToMerge).xy;

	// 1:1 pixels so this is safe RMW
	MergePhysicalPixel(BasePos + uint2(0U, 0U));
	MergePhysicalPixel(BasePos + uint2(1U, 0U));
	MergePhysicalPixel(BasePos + uint2(0U, 1U));
	MergePhysicalPixel(BasePos + uint2(1U, 1U));
}

StructuredBuffer<uint> PhysicalPagesWithFilterable;
StructuredBuffer<uint> PhysicalPagesWithSeparableFilterable;

float CalculateOcclusion(float ShadowMapDepth, float TestDepth, float TransitionScale)
{
#if 0
	// TODO: do the falloff thing?
	if (ShadowMapDepth < TestDepth)
	{
		return 1.0f;
	}
	float Alpha = ShadowMapDepth - TestDepth;
	return 1.0f - saturate(Alpha * TransitionScale);
#else
	if (ShadowMapDepth > TestDepth)
	{
		return 0.0f;
	}
	return 1.0f;
#endif
}

#define SEPARABLE_KERNEL_EXT (2)
#define SEPARABLE_DISABLE_BLUR 0

// load border to enable sampling neighbors in the filter kernel
groupshared float TmpRowCol[VSM_PAGE_SIZE + SEPARABLE_KERNEL_EXT * 2];

struct FBorderSampleData
{
	bool bSrcPosValid;
	uint2 SrcPos;
	int DestTempIndex;
};

FBorderSampleData CalcBorderSampleCoord(FVirtualSMLevelOffset Level0Offset, uint GroupThreadIndex, uint2 vPageIn, uint RowOrCol, int2 Axis)
{
	FBorderSampleData BorderSampleData = (FBorderSampleData)0;
	// 2. Load border samples using subset of threads.
	if (GroupThreadIndex < SEPARABLE_KERNEL_EXT * 2)
	{
		// which border? 0 for lower 1 for upper
		int BorderId = GroupThreadIndex / SEPARABLE_KERNEL_EXT;
		int2 vPage = int2(vPageIn) + Axis * int((BorderId << 1) - 1);
			
		// index of the border item
		BorderSampleData.DestTempIndex = (GroupThreadIndex % SEPARABLE_KERNEL_EXT) + (VSM_PAGE_SIZE + SEPARABLE_KERNEL_EXT) * BorderId;

		uint2 pPage = (uint2)0;
		if (VirtualToPhysicalPage(Level0Offset, 0u, vPage, pPage))
		{
			uint2 PhysPagePos = pPage << VSM_LOG2_PAGE_SIZE;
			// map into the neighbor page space
			uint SrcPageCoordXY = (VSM_PAGE_SIZE - SEPARABLE_KERNEL_EXT) * (1 - BorderId) + (GroupThreadIndex % SEPARABLE_KERNEL_EXT);
			uint2 Offset = uint2(SrcPageCoordXY, RowOrCol);
			if (Axis.y)
			{
				Swap(Offset.x, Offset.y);
			}
			BorderSampleData.SrcPos = PhysPagePos + Offset;
			BorderSampleData.bSrcPosValid = true;
		}
		else
		{
			// no neighbor page, clamp to border
			int BorderIndex = SEPARABLE_KERNEL_EXT + (VSM_PAGE_SIZE - 1u) * BorderId;
			TmpRowCol[BorderSampleData.DestTempIndex] = TmpRowCol[BorderIndex];
		}
	}
	return BorderSampleData;
}

#ifdef UpdateFilterablePagesSeparableX_CS

RWTexture2D<float> OutPrefilteredDistantPhysicalPagePoolX;


// one thread per texel
[numthreads(128, 1, 1)]
void UpdateFilterablePagesSeparableX_CS(uint GroupThreadIndex : SV_GroupIndex, uint GroupIndex : SV_GroupID)
{
	// one group per row (could loop instead/also which might be better for thread launch cost?
	uint ItemIndex = GroupIndex >> VSM_LOG2_PAGE_SIZE;
	uint PhysicalPageIndex = PhysicalPagesWithSeparableFilterable[ItemIndex];
	FPhysicalPageMetaData MetaData = PhysicalPageMetaData[PhysicalPageIndex];
	// Sample neighbor pages if they exist:
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(MetaData.VirtualShadowMapId);
	// Offset to the virtual page table.
	FVirtualSMLevelOffset Level0Offset = CalcPageTableLevelOffset(VirtualShadowMapHandle, 0u);
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	uint2 CurrentPhysPageAddress = VSMPhysicalIndexToPageAddress(PhysicalPageIndex);
	uint2 CurrentPhysPagePos = CurrentPhysPageAddress << VSM_LOG2_PAGE_SIZE;
	{
		uint Row = GroupIndex & VSM_PAGE_SIZE_MASK;

		const float TestDepth = ProjectionData.DynamicDepthCullRange.x;
		// Unused for now
		const float TransitionScale = 1.0f;//ProjectionData.FilterableTransitionScale;

		uint2 CurrentPhysPos = CurrentPhysPagePos + uint2(GroupThreadIndex, Row);
		// 1. load the inner samples
		{

			float Depth = asfloat(VirtualShadowMap.PhysicalPagePool.Load(uint4(CurrentPhysPos, 0, 0)));
			float Occlusion = CalculateOcclusion(Depth, TestDepth, TransitionScale);
			TmpRowCol[SEPARABLE_KERNEL_EXT + GroupThreadIndex] = Occlusion;
		}
		GroupMemoryBarrierWithGroupSync();
		// 2. Load border samples using subset of threads.
		FBorderSampleData BSD = CalcBorderSampleCoord(Level0Offset, GroupThreadIndex, MetaData.PageAddress, Row, int2(1,0));
		if (BSD.bSrcPosValid)
		{
			float Depth = asfloat(VirtualShadowMap.PhysicalPagePool.Load(uint4(BSD.SrcPos, 0, 0)));
			float Occlusion = CalculateOcclusion(Depth, TestDepth, TransitionScale);
			TmpRowCol[BSD.DestTempIndex] = Occlusion;
		}
		GroupMemoryBarrierWithGroupSync();

		// Do 1-D filter
		const float FilterWeights[3] = { 1.0f,  4.0f,  6.0f };
		const int K = SEPARABLE_KERNEL_EXT;

#if SEPARABLE_DISABLE_BLUR
		float Result = TmpRowCol[GroupThreadIndex + SEPARABLE_KERNEL_EXT] * 16.0f;
#else
		float Result = 0.0f;
		for (int X = -K; X <= K; ++X )
		{
			float W = FilterWeights[K - abs(X)];
			float Occlusion = TmpRowCol[GroupThreadIndex + SEPARABLE_KERNEL_EXT + X];
			Result += W * Occlusion;
		}
#endif	
		uint2 DstPos = CurrentPhysPos;
		// horizontal pass output
		OutPrefilteredDistantPhysicalPagePoolX[DstPos] = Result * (1.0f / 16.0f);;
	}
}

#endif // UpdateFilterablePagesSeparableX_CS

#ifdef UpdateFilterablePagesSeparableY_CS

Texture2D<float> PrefilteredDistantPhysicalPagePoolX;
RWTexture2D<float> OutPrefilteredDistantPhysicalPagePool;

Texture2D<float> PrevPrefilteredDistantPhysicalPagePool;
Texture2D<uint> PrevPageReceiverMasks;
ByteAddressBuffer PrevProjectionData;
Texture2D<uint> PrevPageTable;
float HistoryBlendFactor;
uint bHavePrevReceiverMasks;

StructuredBuffer<uint> PrevVirtualShadowMapIds;

bool SampleVirtualShadowMapDirectionalPrefilteredDistantMappedPrev(FVirtualShadowMapHandle MappedClipmapHandle, float3 TranslatedWorldPositionView, inout float OutShadowFactor)
{
	float2 VirtualTexelAddressFloat = ViewTranslatedWorldToVirtualTexel(GetVirtualShadowMapProjectionData(MappedClipmapHandle, PrevProjectionData), TranslatedWorldPositionView);

	FVirtualShadowMapFilterableContext Context;
	Context.PrefilteredDistantPhysicalPagePool = PrevPrefilteredDistantPhysicalPagePool;
	Context.PageTable = PrevPageTable;
	Context.bHavePrevReceiverMasks = bHavePrevReceiverMasks != 0u;
	Context.PageReceiverMasks = PrevPageReceiverMasks;

	FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(MappedClipmapHandle, 0u);
	return Context.SampleBilinearReceiverMask(PageTableLevelOffset, VirtualTexelAddressFloat, OutShadowFactor);
}

bool SampleVirtualShadowMapDirectionalPrefilteredDistantPrev(FVirtualShadowMapHandle BaseVirtualShadowMapHandle, FDFVector3 WoldPosition,  inout float OutShadowFactor)
{
	FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(BaseVirtualShadowMapHandle, PrevProjectionData);

	float3 PrevTranslatedWorldPositionShadow = DFAddDemote(WoldPosition, BaseProjectionData.PreViewTranslation);
	
	float3 PrevTranslatedWorldPositionView = PrevTranslatedWorldPositionShadow - BaseProjectionData.ClipmapWorldOriginOffset;

	checkSlow(BaseProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL);
	
	// Note: length2(PrevTranslatedWorldPositionView) works (that is, is the same as the distance to the clipmap origin) if
	// 1. The origin is not manipulated by the ortho thing.
	// 2. it is a clipmap
	const float ClipmapLevelFloat = CalcBiasedAbsoluteClipmapLevelForSampling(BaseProjectionData, length2(PrevTranslatedWorldPositionView), -1.0f);
	const int ClipmapLevel = int(floor(ClipmapLevelFloat));
	int ClipmapIndex = max(0, ClipmapLevel - BaseProjectionData.ClipmapLevel);
	{
		FVirtualShadowMapHandle ClipmapLevelVirtualShadowMapHandle = BaseVirtualShadowMapHandle.MakeOffset(ClipmapIndex);
		return SampleVirtualShadowMapDirectionalPrefilteredDistantMappedPrev(ClipmapLevelVirtualShadowMapHandle, PrevTranslatedWorldPositionView, OutShadowFactor);
	}
	return false;
}

// one thread per texel
[numthreads(128, 1, 1)]
void UpdateFilterablePagesSeparableY_CS(uint GroupThreadIndex : SV_GroupIndex, uint GroupIndex : SV_GroupID)
{
	// one group per row (could loop instead/also which might be better for thread launch cost?
	uint ItemIndex = GroupIndex >> VSM_LOG2_PAGE_SIZE;
	uint Col = GroupIndex & VSM_PAGE_SIZE_MASK;
	uint PhysicalPageIndex = PhysicalPagesWithSeparableFilterable[ItemIndex];
	FPhysicalPageMetaData MetaData = PhysicalPageMetaData[PhysicalPageIndex];

	uint2 CurrentPhysPageAddress = VSMPhysicalIndexToPageAddress(PhysicalPageIndex);
	uint2 CurrentTempPhysPagePos = CurrentPhysPageAddress << VSM_LOG2_PAGE_SIZE;
	uint2 CurrentTempPhysPos = CurrentTempPhysPagePos + uint2(Col, GroupThreadIndex);

	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(MetaData.VirtualShadowMapId);
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

	{
		// 1. load the inner samples
		TmpRowCol[SEPARABLE_KERNEL_EXT + GroupThreadIndex] = PrefilteredDistantPhysicalPagePoolX.Load(uint3(CurrentTempPhysPos, 0));
		GroupMemoryBarrierWithGroupSync();

		// Offset to the virtual page table.
		FVirtualSMLevelOffset Level0Offset = CalcPageTableLevelOffset(VirtualShadowMapHandle, 0u);
		// 2. Load border samples using subset of threads.
		FBorderSampleData BSD = CalcBorderSampleCoord(Level0Offset, GroupThreadIndex, MetaData.PageAddress, Col, int2(0,1));
		if (BSD.bSrcPosValid)
		{
			TmpRowCol[BSD.DestTempIndex] = PrefilteredDistantPhysicalPagePoolX.Load(uint3(BSD.SrcPos, 0));
		}

		GroupMemoryBarrierWithGroupSync();

		// Do 1-D filter
		const float FilterWeights[3] = { 1.0f,  4.0f,  6.0f };
		const int K = SEPARABLE_KERNEL_EXT;

#if SEPARABLE_DISABLE_BLUR
		float Result = TmpRowCol[GroupThreadIndex + SEPARABLE_KERNEL_EXT] * 16.0f;
#else
		float Result = 0.0f;
		for (int X = -K; X <= K; ++X )
		{
			float W = FilterWeights[K - abs(X)];
			Result += W * TmpRowCol[GroupThreadIndex + SEPARABLE_KERNEL_EXT + X];
		}
#endif
		float ShadowFactor = Result * (1.0f / 16.0f);
		GroupMemoryBarrierWithGroupSync();

		if (HistoryBlendFactor > 0.0f)
		{
			int BaseId = MetaData.VirtualShadowMapId - int(ProjectionData.ClipmapLevelIndex);
			int PrevBaseVirtualShadowMapId = int(PrevVirtualShadowMapIds[BaseId]) - 1;
			if (PrevBaseVirtualShadowMapId >= 0)
			{
				// Sample and blend previous frame

				// 1. What is my translated world position?
				uint2 vPage = MetaData.PageAddress;
				uint2 vTexel = vPage * VSM_PAGE_SIZE + uint2(Col, GroupThreadIndex); 
				float2 vUV = (float2(vTexel) + 0.5f) / float(VSM_VIRTUAL_MAX_RESOLUTION_XY);
				float Z = (ProjectionData.DynamicDepthCullRange.x + ProjectionData.DynamicDepthCullRange.y) * 0.5f;

				float4 TranslatedWorldPositionViewHom = mul(float4(vUV, Z, 1.0f), ProjectionData.ShadowUVToTranslatedWorldMatrix);

				// 1. figure out TLW in primary view of current frame
				float3 CurrentShadowTranslatedWorldPosition = TranslatedWorldPositionViewHom.xyz / TranslatedWorldPositionViewHom.w;
				FDFVector3 CurrentWorldPos  = DFSubtract(CurrentShadowTranslatedWorldPosition, ProjectionData.PreViewTranslation);
				// 2. Figure out TLW in prev frame view.
				// Sample previous SM
				FVirtualShadowMapHandle PrevBaseVirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(PrevBaseVirtualShadowMapId);
				float PrevShadowFactor = 0.0f;
				if (SampleVirtualShadowMapDirectionalPrefilteredDistantPrev(PrevBaseVirtualShadowMapHandle, CurrentWorldPos, PrevShadowFactor))
				{
					float EffectiveBlendFactor = HistoryBlendFactor;

					// ShadowFactor = lerp(ShadowFactor, PrevShadowFactor, EffectiveBlendFactor);

					float Delta = (ShadowFactor - PrevShadowFactor) * (1.0f - EffectiveBlendFactor);
					if (abs(Delta) < 1.0f / 255.0f)
					{
						Delta = sign(Delta) / 255.0f;
					}
					ShadowFactor = saturate(PrevShadowFactor + Delta);
				}
			}
		}

		// vertical pass output
		uint2 CurrentPhysPagePos = CurrentPhysPageAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE;
		uint2 CurrentPhysPos = CurrentPhysPagePos + uint2(Col, GroupThreadIndex) + VSM_PREFILTERED_DISTANT_BORDER_SIZE;
		OutPrefilteredDistantPhysicalPagePool[CurrentPhysPos] = ShadowFactor;
	}
}

#endif // UpdateFilterablePagesSeparableY_CS

#ifdef GatherFilterableBorderTexels_CS

RWTexture2D<float> InOutPrefilteredDistantPhysicalPagePool;

void MapToNeighborClamp(FVirtualSMLevelOffset PageTableLevelOffset, int2 Dir, uint2 DestPPage, uint2 DestVPage, inout uint2 DstPhysBaseAddress, inout uint2 SrcPhysBaseAddress)
{
	// default to copy own row to "clamp to edge"
	int2 SrcVPage = int2(DestVPage) + Dir;
	uint2 SrcPPage = DestPPage;
	uint2 Offset = uint2(max(Dir, 0) * (VSM_PAGE_SIZE - 1u));
	uint2 SrcOffset = Offset;
	if (VirtualToPhysicalPage(PageTableLevelOffset, 0u, SrcVPage, SrcPPage))
	{
		SrcOffset = uint2(max(-Dir, 0) * (VSM_PAGE_SIZE - 1u));
	}
	// SrcPhysBaseAddress += uint2(GroupThreadIndex, 0u);
		
	DstPhysBaseAddress = DestPPage * VSM_PREFILTERED_DISTANT_PAGE_SIZE + uint2(VSM_PREFILTERED_DISTANT_BORDER_SIZE, VSM_PREFILTERED_DISTANT_BORDER_SIZE) + Offset + Dir;
	SrcPhysBaseAddress = SrcPPage * VSM_PREFILTERED_DISTANT_PAGE_SIZE + uint2(VSM_PREFILTERED_DISTANT_BORDER_SIZE, VSM_PREFILTERED_DISTANT_BORDER_SIZE) + SrcOffset;
};

void CopyBorderRowColClamp(FVirtualSMLevelOffset PageTableLevelOffset, int2 Dir, uint2 DestPPage, uint2 DestVPage, uint GroupThreadIndex)
{
	uint2 SrcPhysBaseAddress;
	uint2 DstPhysBaseAddress;
	MapToNeighborClamp(PageTableLevelOffset, Dir, DestPPage, DestVPage, DstPhysBaseAddress, SrcPhysBaseAddress);
	uint2 ThreadOffset = abs(Dir).yx * GroupThreadIndex;
	InOutPrefilteredDistantPhysicalPagePool[DstPhysBaseAddress + ThreadOffset] = InOutPrefilteredDistantPhysicalPagePool[SrcPhysBaseAddress + ThreadOffset];
}

[numthreads(128, 1, 1)]
void GatherFilterableBorderTexels_CS(uint GroupThreadIndex : SV_GroupIndex, uint GroupIndex : SV_GroupID)
{
	// one group per VSM
	uint ItemIndex = GroupIndex;
	uint Col = GroupIndex & VSM_PAGE_SIZE_MASK;
	uint PhysicalPageIndex = PhysicalPagesWithSeparableFilterable[ItemIndex];
	FPhysicalPageMetaData MetaData = PhysicalPageMetaData[PhysicalPageIndex];

	uint2 CurrentPhysPageAddress = VSMPhysicalIndexToPageAddress(PhysicalPageIndex);
	uint2 CurrentPhysPagePos = CurrentPhysPageAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE + VSM_PREFILTERED_DISTANT_BORDER_SIZE;
	//uint2 CurrentPhysPos = 
	

	uint2 DestVPage = MetaData.PageAddress;

	//OutPrefilteredDistantPhysicalPagePool[CurrentPhysPos] = ShadowFactor;

	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(MetaData.VirtualShadowMapId);
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(VirtualShadowMapHandle, 0u);

	// 1. copy the border in Y directions
	CopyBorderRowColClamp(PageTableLevelOffset, int2(0, -1), CurrentPhysPageAddress, DestVPage, GroupThreadIndex);
	CopyBorderRowColClamp(PageTableLevelOffset, int2(0,  1), CurrentPhysPageAddress, DestVPage, GroupThreadIndex);
	// 2. copy the border in X directions
	CopyBorderRowColClamp(PageTableLevelOffset, int2(-1, 0), CurrentPhysPageAddress, DestVPage, GroupThreadIndex);
	CopyBorderRowColClamp(PageTableLevelOffset, int2( 1, 0), CurrentPhysPageAddress, DestVPage, GroupThreadIndex);


	// 3. copy the 4 corners in diagonal directions using one thread each
	if (GroupThreadIndex < 4u)
	{
		int2 Dir = int2(GroupThreadIndex % 2, GroupThreadIndex / 2) * 2 - 1;

		uint2 SrcPhysBaseAddress;
		uint2 DstPhysBaseAddress;
		MapToNeighborClamp(PageTableLevelOffset, Dir, CurrentPhysPageAddress, DestVPage, DstPhysBaseAddress, SrcPhysBaseAddress);
		InOutPrefilteredDistantPhysicalPagePool[DstPhysBaseAddress] = InOutPrefilteredDistantPhysicalPagePool[SrcPhysBaseAddress];
	}
}

#endif // GatherFilterableBorderTexels_CS

#ifdef MergeDownFilterablePagesCS

RWTexture2D<float> OutPrefilteredDistantPhysicalPagePool;
Texture2D<float> PrefilteredDistantPhysicalPagePool;
Texture2D<uint> PrefilteredDistantPhysicalPagePoolPacked;

void AccumulateShadowFactor(FVirtualShadowMapFilterableContext Context, FShadowPhysicalPage ShadowPhysicalPage, float2 VirtualTexelPos, inout float OutAccumulator)
{
	// TODO: consider trying Texture2DSampleBicubic or otherwise implement bicubic
	float ShadowFactor = 0.0f;
	Context.SampleBilinear(ShadowPhysicalPage, VirtualTexelPos, ShadowFactor);
	OutAccumulator *= ShadowFactor;
}

groupshared uint PageCacheGS[32];
groupshared float2 BiasGS[32];

[numthreads(TILE_THREAD_GROUP_SIZE_XY, TILE_THREAD_GROUP_SIZE_XY, 1)]
void MergeDownFilterablePagesCS(uint2 TileThreadID : SV_GroupThreadID, uint GroupIndex : SV_GroupID)
{
	FTileSetup TileSetup = GetTileSetup(GroupIndex, PhysicalPagesWithFilterable);
	FVirtualShadowMapHandle DstVirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(TileSetup.MetaData.VirtualShadowMapId);
	FVirtualShadowMapProjectionShaderData DestProjectionData = GetVirtualShadowMapProjectionData(DstVirtualShadowMapHandle);

	const uint2 PageSpaceBasePos = TileSetup.TileOffsetInPage.xy + TileThreadID.xy;
	const uint SampleStride = TILE_THREAD_GROUP_SIZE_XY;
	const uint2 DstBasePos = (TileSetup.PhysPageAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE) + PageSpaceBasePos + VSM_PREFILTERED_DISTANT_BORDER_SIZE;

	// 2x2 result for the thread
	float Tmp00 = 1.0f;
	float Tmp10 = 1.0f;
	float Tmp01 = 1.0f;
	float Tmp11 = 1.0f;

	uint2 DestVPage = TileSetup.MetaData.PageAddress;
	// // Map destination page to common root-space
	const int OffsetScale = (VSM_VIRTUAL_MAX_RESOLUTION_XY >> 2);
	float2 DestOffset = float2(DestProjectionData.ClipmapCornerRelativeOffset * OffsetScale);

	FVirtualShadowMapFilterableContext Ctx;
	Ctx.PrefilteredDistantPhysicalPagePool = PrefilteredDistantPhysicalPagePool;
	Ctx.PageTable = VirtualShadowMap.PageTable;

	uint LevelEnd = DestProjectionData.ClipmapLevelCountRemaining;//
	// uint LevelEnd = min(1, DestProjectionData.ClipmapLevelCountRemaining);//DestProjectionData.ClipmapLevelCountRemaining;//

	// Unroll page-table lookups using threads & store in GS
	uint ClipmapLevelOffset = TileThreadID.y * TILE_THREAD_GROUP_SIZE_XY + TileThreadID.x;
	ClipmapLevelOffset += 1u;
	if (LevelEnd > 1u)
	{
		if (ClipmapLevelOffset < LevelEnd)
		{
			FVirtualShadowMapHandle SourceLevelHandle = DstVirtualShadowMapHandle.MakeOffset(ClipmapLevelOffset);
			FVirtualShadowMapProjectionShaderData SourceProjectionData = GetVirtualShadowMapProjectionData(SourceLevelHandle);
		
			uint InvLevelScale = 1u << ClipmapLevelOffset;
			float LevelScale = 1.0f / float(InvLevelScale);

			float2 SrcOffset = float2(SourceProjectionData.ClipmapCornerRelativeOffset * OffsetScale);
			float2 Bias = SrcOffset - DestOffset * LevelScale;

			// uses representative texel
			float2 DstVTexel = float2((DestVPage << VSM_LOG2_PAGE_SIZE) + TileSetup.TileOffsetInPage);
			float2 SrcVTexel = DstVTexel * LevelScale + Bias;

			uint2 SrcVPage = uint2(SrcVTexel) >> VSM_LOG2_PAGE_SIZE;

			FVSMPageOffset VSMPageOffset = CalcPageOffset(SourceLevelHandle, 0u, SrcVPage);
			uint PhysicalPage = VirtualShadowMap.PageTable[VSMPageOffset.GetResourceAddress()];

			PageCacheGS[ClipmapLevelOffset] = PhysicalPage;
			BiasGS[ClipmapLevelOffset] = Bias;

		}
		GroupMemoryBarrierWithGroupSync();
	}

	// 1. Destination virtual address (Note: full-res texel address, not HZB)
	float2 DstVTexelUint = (DestVPage << VSM_LOG2_PAGE_SIZE) + PageSpaceBasePos;
	float2 DstVTexel = float2(DstVTexelUint) + 0.5f;

	// Manually unroll first iteration, needs no translation etc
	BRANCH
	if (!(TileSetup.MetaData.Flags & VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED))
	{
		uint2 PhysicalTexelAddress = DstBasePos;

		Tmp00 = PrefilteredDistantPhysicalPagePool[PhysicalTexelAddress + uint2(0u,				0u)];
		Tmp10 = PrefilteredDistantPhysicalPagePool[PhysicalTexelAddress + uint2(SampleStride,	0u)];
		Tmp01 = PrefilteredDistantPhysicalPagePool[PhysicalTexelAddress + uint2(0u,				SampleStride)];
		Tmp11 = PrefilteredDistantPhysicalPagePool[PhysicalTexelAddress + uint2(SampleStride,	SampleStride)];
	}
	uint StartLevel = 1u;
	float LevelScale = 0.5f;
	for (uint ClipmapLevelOffset = StartLevel; ClipmapLevelOffset < LevelEnd; ++ClipmapLevelOffset)
	{
		float2 TexelJitterOffset = (float2)0;
		// 2. scale and bias to the source space
		float2 Bias = BiasGS[ClipmapLevelOffset];
		float2 SrcVTexel = DstVTexel * LevelScale + Bias;

		FShadowPhysicalPage ShadowPhysicalPage = ShadowDecodePageTable(PageCacheGS[ClipmapLevelOffset]);
		BRANCH
		if (ShadowPhysicalPage.bThisLODValid)
		{
			FPhysicalPageMetaData MetaData = PhysicalPageMetaData[VSMPhysicalPageAddressToIndex(ShadowPhysicalPage.PhysicalAddress)];

			BRANCH
			if (!(MetaData.Flags & VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED))
			{
				AccumulateShadowFactor(Ctx, ShadowPhysicalPage, SrcVTexel + (float2(0.0f,			0.0f)		  + TexelJitterOffset) * LevelScale, Tmp00);
				AccumulateShadowFactor(Ctx, ShadowPhysicalPage, SrcVTexel + (float2(SampleStride,	0.0f)		  + TexelJitterOffset) * LevelScale, Tmp10);
				AccumulateShadowFactor(Ctx, ShadowPhysicalPage, SrcVTexel + (float2(0.0f,			SampleStride) + TexelJitterOffset) * LevelScale, Tmp01);
				AccumulateShadowFactor(Ctx, ShadowPhysicalPage, SrcVTexel + (float2(SampleStride,	SampleStride) + TexelJitterOffset) * LevelScale, Tmp11);
			}
		}
		LevelScale *= 0.5f;
	}
	OutPrefilteredDistantPhysicalPagePool[DstBasePos + uint2(0U,			0U)]			= Tmp00;
	OutPrefilteredDistantPhysicalPagePool[DstBasePos + uint2(SampleStride,	0U)]			= Tmp10;
	OutPrefilteredDistantPhysicalPagePool[DstBasePos + uint2(0U,			SampleStride)]	= Tmp01;
	OutPrefilteredDistantPhysicalPagePool[DstBasePos + uint2(SampleStride,	SampleStride)]	= Tmp11;
}

#endif // MergeDownFilterablePagesCS

// Returns updated physical page flags
uint UpdateAndClearDirtyFlags(uint PhysicalPageIndex)
{
	bool bDynamicDirty 		   = DirtyPageFlagsInOut[PhysicalPageIndex + 0U * VirtualShadowMap.MaxPhysicalPages] != 0U;
	bool bStaticDirty 		   = DirtyPageFlagsInOut[PhysicalPageIndex + 1U * VirtualShadowMap.MaxPhysicalPages] != 0U;
	bool bInvalidatesDynamic   = DirtyPageFlagsInOut[PhysicalPageIndex + 2U * VirtualShadowMap.MaxPhysicalPages] != 0U;
	bool bInvalidatesStatic    = DirtyPageFlagsInOut[PhysicalPageIndex + 3U * VirtualShadowMap.MaxPhysicalPages] != 0U;
	bool bWPOAllowed           = DirtyPageFlagsInOut[PhysicalPageIndex + 4U * VirtualShadowMap.MaxPhysicalPages] != 0U;
	bool bDeferredInvalidation = DirtyPageFlagsInOut[PhysicalPageIndex + 5U * VirtualShadowMap.MaxPhysicalPages] != 0U;

	// Clear the dirty/invalidation flags
	DirtyPageFlagsInOut[PhysicalPageIndex + 0U * VirtualShadowMap.MaxPhysicalPages] = 0U;
	DirtyPageFlagsInOut[PhysicalPageIndex + 1U * VirtualShadowMap.MaxPhysicalPages] = 0U;
	DirtyPageFlagsInOut[PhysicalPageIndex + 2U * VirtualShadowMap.MaxPhysicalPages] = 0U;
	DirtyPageFlagsInOut[PhysicalPageIndex + 3U * VirtualShadowMap.MaxPhysicalPages] = 0U;
	DirtyPageFlagsInOut[PhysicalPageIndex + 4U * VirtualShadowMap.MaxPhysicalPages] = 0U;
	DirtyPageFlagsInOut[PhysicalPageIndex + 5U * VirtualShadowMap.MaxPhysicalPages] = 0U;

	// Collect/compress them into physical page flags
	uint Flags = OutPhysicalPageMetaData[PhysicalPageIndex].Flags;

	if (Flags != 0)
	{
		// In this path it means we wrote something to the page, so no longer initialized
		if (bDynamicDirty)
		{
			Flags &= ~(VSM_EXTENDED_FLAG_DYNAMIC_INITIALIZED);
			Flags |= VSM_EXTENDED_FLAG_DYNAMIC_DIRTY;
		}
		if (bStaticDirty)
		{
			Flags &= ~(VSM_EXTENDED_FLAG_STATIC_INITIALIZED);
			Flags |= VSM_EXTENDED_FLAG_STATIC_DIRTY;
		}

		Flags |=
			(bInvalidatesDynamic   ? VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC   : 0U) |
			(bInvalidatesStatic    ? VSM_EXTENDED_FLAG_INVALIDATE_STATIC    : 0U) |
			(bWPOAllowed           ? VSM_EXTENDED_FLAG_FORCE_CACHED         : 0U) |
			(bDeferredInvalidation ? VSM_EXTENDED_FLAG_DEFERRED_INVALIDATION : 0U);

		// Update the metadata on the page
		OutPhysicalPageMetaData[PhysicalPageIndex].Flags = Flags;
	}

	return Flags;
}

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void UpdateAndClearDirtyFlagsCS(uint PhysicalPageIndex : SV_DispatchThreadID)
{
	if (PhysicalPageIndex >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}
	FPhysicalPageMetaData MetaData;
	UpdateAndClearDirtyFlags(PhysicalPageIndex);
}

RWBuffer<uint> OutPagesForHZBIndirectArgsBuffer;
RWStructuredBuffer<uint> OutPhysicalPagesForHZB;
uint bFirstBuildThisFrame;
uint bForceFullHZBUpdate;

[numthreads(VSM_DEFAULT_CS_GROUP_X, 1, 1)]
void SelectPagesForHZBAndUpdateDirtyFlagsCS(uint PhysicalPageIndex : SV_DispatchThreadID)
{
	if (PhysicalPageIndex >= VirtualShadowMap.MaxPhysicalPages)
	{
		return;
	}

	uint PhysicalPageFlags = UpdateAndClearDirtyFlags(PhysicalPageIndex);

	if ((PhysicalPageFlags & VSM_FLAG_ALLOCATED) != 0)
	{	
		const bool bStaticDirty  = (PhysicalPageFlags & VSM_EXTENDED_FLAG_STATIC_DIRTY) != 0u;
		const bool bDynamicDirty = (PhysicalPageFlags & VSM_EXTENDED_FLAG_DYNAMIC_DIRTY) != 0u;		
		const bool bUnreferenced = (PhysicalPageFlags & VSM_EXTENDED_FLAG_UNREFERENCED) != 0u;
		const bool bViewUncached = (PhysicalPageFlags & VSM_EXTENDED_FLAG_VIEW_UNCACHED) != 0U;
		const bool bHasSeparateDynamicHZB = VirtualShadowMap.StaticHZBArrayIndex != 0;
		const bool bNeedDynamicBuild = (bViewUncached || bHasSeparateDynamicHZB) && bDynamicDirty;

		// Skip it if it's not referenced; this is usually because we have already done
		// the HZB rebuild but haven't yet cleared these flags, which happens the nexd time
		// a page gets rendered.
		// TODO: We should clear the relevant page flags immediately after doing HZB generation
		bool bRebuildHZB = (bStaticDirty || bNeedDynamicBuild) && !bUnreferenced;

		if (bForceFullHZBUpdate || bRebuildHZB)
		{
			StatsBufferInterlockedInc(VSM_STAT_NUM_HZB_PAGES_BUILT);

			int GroupCount = 0;
			// Each page needs TILES_PER_PAGE_1D groups launched
			WaveInterlockedAddScalar_(OutPagesForHZBIndirectArgsBuffer[0], TILES_PER_PAGE_1D, GroupCount);
			OutPhysicalPagesForHZB[GroupCount >> LOG2_TILES_PER_PAGE_1D] = PhysicalPageIndex;

			// Each top-reduction needs only one group launched
			WaveInterlockedAddScalar_(OutPagesForHZBIndirectArgsBuffer[0 + 4], 1U, GroupCount);
		}
	}
}


SamplerState PhysicalPagePoolSampler;
Texture2DArray<uint> PhysicalPagePool;

float4 Gather4VisZ(uint2 PixelCoord, uint ArrayIndex)
{
#if COMPILER_SUPPORTS_GATHER_UINT
	// Offset to 2x2 footprint center and scale to UV space
	float2 UV = float2(PixelCoord + uint2(1U, 1U)) * VirtualShadowMap.RecPhysicalPoolSize.xy;
	return asfloat(PhysicalPagePool.Gather(PhysicalPagePoolSampler, float3(UV, ArrayIndex), 0));
#else
	uint4 PixelRect = uint4(PixelCoord.xy, PixelCoord.xy + uint2(1U, 1U));
	uint4 UintDepths = uint4(
		PhysicalPagePool[uint3(PixelRect.xw, ArrayIndex)].r,	// (-, +)
		PhysicalPagePool[uint3(PixelRect.zw, ArrayIndex)].r,	// (+, +)
		PhysicalPagePool[uint3(PixelRect.zy, ArrayIndex)].r,	// (+, -)
		PhysicalPagePool[uint3(PixelRect.xy, ArrayIndex)].r		// (-, -)
	);
	return asfloat(UintDepths);
#endif
}

StructuredBuffer<uint> PhysicalPagesForHzb;
//                                         out                input               output   
RWTexture2DArray<float> FurthestHZBArrayOutput_0; // 64 // 1     Group: 32 (16 threads x2)  16
RWTexture2DArray<float> FurthestHZBArrayOutput_1; // 32 // 1            16                  8
RWTexture2DArray<float> FurthestHZBArrayOutput_2; // 16                 8                   4
RWTexture2DArray<float> FurthestHZBArrayOutput_3; // 8                  4                   2
RWTexture2DArray<float> FurthestHZBArrayOutput_4; // 4                  2                   1

groupshared float SharedMinDeviceZ[TILE_THREAD_GROUP_SIZE_XY * TILE_THREAD_GROUP_SIZE_XY];
groupshared float SharedMaxDeviceZ[TILE_THREAD_GROUP_SIZE_XY * TILE_THREAD_GROUP_SIZE_XY];

#define DIM_FURTHEST 1
#define DIM_CLOSEST 0

void OutputMipLevel(uint MipLevel, uint2 OutputPixelPos, int ArrayIndex, float FurthestDeviceZ, float ClosestDeviceZ)
{
#if DIM_FURTHEST
#define COND_OUTPUT_LEVEL(_level_) \
	if (MipLevel == _level_) \
	{ \
		FurthestHZBArrayOutput_##_level_[uint3(OutputPixelPos, ArrayIndex)] = FurthestDeviceZ; \
		return; \
	}
#endif
#if DIM_CLOSEST
		ClosestHZBOutput_1[uint3(OutputPixelPos, ArrayIndex)] = ClosestDeviceZ;
#endif

COND_OUTPUT_LEVEL(1)
COND_OUTPUT_LEVEL(2)
COND_OUTPUT_LEVEL(3)
COND_OUTPUT_LEVEL(4)

#undef COND_OUTPUT_LEVEL
}

void BuildHZBPerPage(uint2 SrcPos, uint GroupThreadIndex, uint HZBArrayIndex, uint SrcArrayIndex, inout float4 InOutDeviceZ)
{
	// Sample 2x2 footprint - thread group covers 32x32 area
	// Merge with static (represented in the InOutDeviceZ)
	float4 DeviceZ = max(InOutDeviceZ, Gather4VisZ(SrcPos, SrcArrayIndex));
	// return the merged result
	InOutDeviceZ = DeviceZ;
	float MinDeviceZ = min(min3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);

	float MaxDeviceZ = 0.0f;//max(max3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);
	//uint LinearGroupThreadID = RemappedGroupThreadIndex.y << LOG2_TILE_THREAD_GROUP_SIZE_XY + RemappedGroupThreadIndex.x;
	
	// Broadcast to all threads (16x16).
	SharedMinDeviceZ[GroupThreadIndex] = MinDeviceZ;
	// Write base HZB level (half physical page size, e.g., 64x64) 
	uint2 OutPixelPos = SrcPos >> 1U;
	FurthestHZBArrayOutput_0[uint3(OutPixelPos, HZBArrayIndex)] = MinDeviceZ;

#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS
		const uint LaneCount = WaveGetLaneCount();
#else
		// Actual wave size is unknown, assume the worst
		const uint LaneCount = 0u;
#endif

	// Build next 4 levels: 32, 16, 8, 4
	UNROLL
	for (uint MipLevel = 1U; MipLevel < LOG2_TILE_SIZE_XY; ++MipLevel)
	{
		// 8x8, 4x4, 2x2, 1x1
		const uint OutTileDim = uint(TILE_THREAD_GROUP_SIZE_XY) >> MipLevel;
		const uint ReduceBankSize = OutTileDim * OutTileDim;
			
		// More waves than one wrote to LDS, need to sync.
		if ((ReduceBankSize << 2u) > LaneCount)
		{
			GroupMemoryBarrierWithGroupSync();
		}

		BRANCH
		if (GroupThreadIndex < ReduceBankSize)
		{
			float4 ParentMinDeviceZ;
			//float4 ParentMaxDeviceZ;
			ParentMinDeviceZ[0] = MinDeviceZ;
			//ParentMaxDeviceZ[0] = MaxDeviceZ;

			UNROLL
			for (uint i = 1; i < 4; i++)
			{
				uint LDSIndex = GroupThreadIndex + i * ReduceBankSize;
				ParentMinDeviceZ[i] = SharedMinDeviceZ[LDSIndex];
				//ParentMaxDeviceZ[i] = SharedMaxDeviceZ[LDSIndex];
			}
				
			MinDeviceZ = min(min3(ParentMinDeviceZ.x, ParentMinDeviceZ.y, ParentMinDeviceZ.z), ParentMinDeviceZ.w);
			//MaxDeviceZ = max(max3(ParentMaxDeviceZ.x, ParentMaxDeviceZ.y, ParentMaxDeviceZ.z), ParentMaxDeviceZ.w);
	
			OutPixelPos = OutPixelPos >> 1;
			OutputMipLevel(MipLevel, OutPixelPos, HZBArrayIndex, MinDeviceZ, MaxDeviceZ);
				
			SharedMinDeviceZ[GroupThreadIndex] = MinDeviceZ;
			//SharedMaxDeviceZ[GroupThreadIndex] = MaxDeviceZ;
		}
	}
}

[numthreads(TILE_THREAD_GROUP_SIZE_XY, TILE_THREAD_GROUP_SIZE_XY, 1)]
void BuildHZBPerPageCS(uint GroupThreadIndex : SV_GroupIndex, uint GroupIndex : SV_GroupID)
{
	FPhysicalPageMetaData MetaData;
	uint2 SrcTileOffset = GetTileOffset(GroupIndex, PhysicalPagesForHzb, MetaData).xy;
	uint2 RemappedGroupThreadIndex = InitialTilePixelPositionForReduction2x2(LOG2_TILE_SIZE_XY - 1U, GroupThreadIndex);
	uint2 SrcPos = SrcTileOffset + (RemappedGroupThreadIndex << uint2(1U, 1U));

	// 1. build the static HZB slice (src slice 1 if enabled)
	uint FirstSrcArrayIndex = GetVirtualShadowMapStaticArrayIndex();
	uint FirstHZBArrayIndex = VirtualShadowMap.StaticHZBArrayIndex;

	bool bViewUncached = (MetaData.Flags & VSM_EXTENDED_FLAG_VIEW_UNCACHED) != 0U;
	// uncachable views always draw to the dynamic slice (slice 0), and thus there is no reason to build for the static slices
	if (bViewUncached)
	{
		FirstSrcArrayIndex = 0u;
		FirstHZBArrayIndex = 0u;
	}
	// 1. build for the static pages (or only if there is only one) & keep the 2x2 device Z to pass to the dynamic build (merge depths in case of both being rebuilt)
	float4 StaticDeviceZ = (float4)0.0f;
	BuildHZBPerPage(SrcPos, GroupThreadIndex, FirstHZBArrayIndex, FirstSrcArrayIndex, StaticDeviceZ);

	// If we have not built the 0th slice, we need to do that also (it must then be the dynamic things)
	// This also covers the case where the HZB only has one slice (and we thus do not build a dynamic one at all).
	if (FirstHZBArrayIndex > 0u)
	{
		BuildHZBPerPage(SrcPos, GroupThreadIndex, 0u, 0u, StaticDeviceZ);
	}
}

float4 Gather4(Texture2DArray Texture, SamplerState TextureSampler, uint2 SrcPos, uint ArrayIndex, float2 InvSize)
{
	float2 SrcUV = float2(SrcPos) * InvSize;
	return Texture.GatherRed(TextureSampler, float3(SrcUV, ArrayIndex), 0);
}

Texture2DArray ParentTextureArrayMip;
SamplerState ParentTextureMipSampler;

float2 InvHzbInputSize;

#define TOP_MIP_TILE_SIZE_XY 4
// Each fetches 2x2 using gather
#define TOP_MIP_TILE_THREAD_GROUP_SIZE_XY (TOP_MIP_TILE_SIZE_XY/2)

void BuildHZBPerPageTop(uint2 SrcPos, uint2 GroupThreadId, uint ArrayIndex)
{
	// Sample 2x2 footprint - thread group covers 32x32 area
	float4 DeviceZ = Gather4(ParentTextureArrayMip, ParentTextureMipSampler, SrcPos + uint2(1U, 1U), ArrayIndex, InvHzbInputSize);
	float MinDeviceZ = min(min3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);

	float MaxDeviceZ = 0.0f;//max(max3(DeviceZ.x, DeviceZ.y, DeviceZ.z), DeviceZ.w);
	//uint LinearGroupThreadID = RemappedGroupThreadIndex.y << LOG2_TILE_THREAD_GROUP_SIZE_XY + RemappedGroupThreadIndex.x;
	
	// Broadcast to all threads.
	SharedMinDeviceZ[GroupThreadId.y * TOP_MIP_TILE_THREAD_GROUP_SIZE_XY + GroupThreadId.x] = MinDeviceZ;
	// Write first HZB output level (half size) 
	uint2 OutPixelPos = SrcPos >> 1U;
	FurthestHZBArrayOutput_0[uint3(OutPixelPos, ArrayIndex)] = MinDeviceZ;

	// Build last level
	GroupMemoryBarrierWithGroupSync();

	BRANCH
	if (all(GroupThreadId.xy == uint2(0U, 0U)))
	{
		float4 ParentMinDeviceZ;
		//float4 ParentMaxDeviceZ;
		ParentMinDeviceZ[0] = MinDeviceZ;
		//ParentMaxDeviceZ[0] = MaxDeviceZ;

		UNROLL
		for (uint Index = 1; Index < 4; ++Index)
		{
			ParentMinDeviceZ[Index] = SharedMinDeviceZ[Index];
			//ParentMaxDeviceZ[i] = SharedMaxDeviceZ[LDSIndex];
		}
				
		MinDeviceZ = min(min3(ParentMinDeviceZ.x, ParentMinDeviceZ.y, ParentMinDeviceZ.z), ParentMinDeviceZ.w);
		//MaxDeviceZ = max(max3(ParentMaxDeviceZ.x, ParentMaxDeviceZ.y, ParentMaxDeviceZ.z), ParentMaxDeviceZ.w);
	
		OutPixelPos = OutPixelPos >> 1;
		FurthestHZBArrayOutput_1[uint3(OutPixelPos, ArrayIndex)] = MinDeviceZ;
	}
}

[numthreads(TOP_MIP_TILE_THREAD_GROUP_SIZE_XY, TOP_MIP_TILE_THREAD_GROUP_SIZE_XY, 1)]
void BuildHZBPerPageTopCS(uint2 GroupThreadId : SV_GroupThreadID, uint PageInputIndex : SV_GroupID)
{
	const uint PageIndex = PhysicalPagesForHzb[PageInputIndex];
	uint2 PhysPageAddress = VSMPhysicalIndexToPageAddress(PageIndex);
	
	// Pixel address of tile region for this thread group.
	const uint2 SrcTileOffset = PhysPageAddress * uint2(TOP_MIP_TILE_SIZE_XY, TOP_MIP_TILE_SIZE_XY);

	uint2 SrcPos = SrcTileOffset +  (GroupThreadId << uint2(1U, 1U));
	BuildHZBPerPageTop(SrcPos, GroupThreadId, 0u);

	const bool bHasSeparateDynamicHZB = VirtualShadowMap.StaticHZBArrayIndex != 0;
	if (bHasSeparateDynamicHZB)
	{
		GroupMemoryBarrierWithGroupSync();
		BuildHZBPerPageTop(SrcPos, GroupThreadId, 1u);
	}
}

uint StatusMessageId;
StructuredBuffer<int> PhysicalPageLists;

[numthreads(1, 1, 1)]
void FeedbackStatusCS()
{
	FGPUMessageWriter Mw = GPUMessageBegin(StatusMessageId, 3U);

	GPUMessageWriteItem(Mw, VSM_STATUS_MSG_PAGE_MANAGEMENT);

	// Write out how many pages are still available
	int CountIndex = GetPhysicalPageListStart(PHYSICAL_PAGE_LIST_AVAILABLE) + VirtualShadowMap.MaxPhysicalPages;
	GPUMessageWriteItem(Mw, PhysicalPageLists[CountIndex]);

	// Write out the resolution lod bias from this frame
	GPUMessageWriteItem(Mw, VirtualShadowMap.GlobalResolutionLodBias);
}



int PageListStatsRow; 

[numthreads(1, 1, 1)]
void LogPageListStatsCS()
{
	float TopMargin = 0.5f;
	float ItemX = 0.05f;
	FShaderPrintContext Ctx = InitShaderPrintContext(true, float2(ItemX, TopMargin));

	Ctx.Pos.y += PageListStatsRow * 0.02f;

	Print(Ctx, GetPhysicalPageListCount(0));
	for (int i = 1; i < PHYSICAL_PAGE_LIST_COUNT; ++i)
	{
		Print(Ctx, TEXT(", "));
		Print(Ctx, GetPhysicalPageListCount(i));
	}
}
