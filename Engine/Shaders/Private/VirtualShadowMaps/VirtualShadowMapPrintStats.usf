// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ShaderPrint.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapStats.ush"
#include "VirtualShadowMapProjectionStructs.ush"
#include "../Nanite/NaniteDataDecode.ush"
#include "../GPUMessaging.ush"
#include "../Visualization.ush"

#define VSM_GENERATE_STATS_GROUPSIZE 128

StructuredBuffer<uint>	InStatsBuffer;
// Rectangles bounding (allocated) pages for each mip level
StructuredBuffer<uint4> AllocatedPageRectBounds;

// From cvar, used to determine which stats are shown
// 1 = all
// 2 = only physical pages
// 3 = only non-nanite-instances
int ShowStatsValue;

void PrintLeftAlign(inout FShaderPrintContext Context, uint Value)
{
	int Len = 9;
	
	uint Tmp = Value;
	while (Tmp >= 10)
	{
		Tmp /= 10;
		Len--;
	}

	while (Len > 0)
	{
		PrintSymbol(Context, _SPC_);
		Len--;
	}
	
	Print(Context, Value);
}

void PrintLeftAlign(inout FShaderPrintContext Context, float Value)
{
	int Len = 9;
	
	uint Tmp = Value;
	while (Tmp >= 10)
	{
		Tmp /= 10;
		Len--;
	}

	while (Len > 0)
	{
		PrintSymbol(Context, _SPC_);
		Len--;
	}
	
	Print(Context, Value);
}

void PrintUnits(inout FShaderPrintContext Context, uint Value)
{
	if (Value >= 1024U * 1024U)
	{
		Print(Context, float((Value * 100U) / (1024U * 1024U)) / 100.0f);
		PrintSymbol(Context, _M_);
	}
	else if (Value >= 1024U)
	{
		Print(Context, float((Value * 100U) / (1024U)) / 100.0f);
		Print(Context, TEXT("k"), FontWhite);
	}
	else
	{
		Print(Context, Value);
	}
}

uint StatusMessageId;

uint StatsMessageId;
uint StatsMessageTimestampHigh;
uint StatsMessageTimestampLow;

StructuredBuffer<uint> NanitePerformanceFeedback;
uint NanitePerformanceFeedbackNumEntries;

StructuredBuffer<uint> ThrottleBuffer;
uint bThrottlingEnabled;

StructuredBuffer<FNaniteStats> NaniteStats;

ByteAddressBuffer PrevProjectionData;

void SendStatusMessage()
{
	FGPUMessageWriter Mw = GPUMessageBegin(StatusMessageId, 2U);
	GPUMessageWriteItem(Mw, VSM_STATUS_MSG_OVERFLOW);
	GPUMessageWriteItem(Mw, InStatsBuffer[VSM_STAT_OVERFLOW_FLAGS]);
}

void SendStatsMessage()
{
	uint MessageSize = 2U + VSM_STAT_NUM + (MAX_PAGE_AREA_DIAGNOSTIC_SLOTS * 2U) + (MAX_NPF_DIAGNOSTIC_SLOTS_SINGLEPAGE + MAX_NPF_DIAGNOSTIC_SLOTS_MULTIPAGE) * 2U;
	FGPUMessageWriter Mw = GPUMessageBegin(StatsMessageId, MessageSize);

	uint ClustersHW = NanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_HW_CLUSTERS];
	uint ClustersSW = NanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_SW_CLUSTERS];

	// Culling stats
	GPUMessageWriteItem(Mw, NaniteStats[0].NumTris);
	GPUMessageWriteItem(Mw, NaniteStats[0].NumMainInstancesPostCull);

	int Stat = 0;
	for (; Stat < VSM_STAT_NANITE_CLUSTERS_HW; ++Stat)
	{
		GPUMessageWriteItem(Mw, InStatsBuffer[Stat]);
	}
	GPUMessageWriteItem(Mw, ClustersHW);
	GPUMessageWriteItem(Mw, ClustersSW);
	for (; Stat < VSM_STAT_NUM; ++Stat)
	{
		GPUMessageWriteItem(Mw, InStatsBuffer[Stat]);
	}

	// Large page area instances
	for (uint Index = 0; Index < MAX_PAGE_AREA_DIAGNOSTIC_SLOTS * 2U; ++Index)
	{
		GPUMessageWriteItem(Mw, InStatsBuffer[VSM_STAT_NUM + Index]);
	}


	for (uint Index = 0; Index < MAX_NPF_DIAGNOSTIC_SLOTS_SINGLEPAGE; ++Index)
	{
		uint BufferIndex = Index;

		uint CurClustersHW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * BufferIndex + VSM_NPF_ENTRY_HW_CLUSTERS];
		uint CurClustersSW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * BufferIndex + VSM_NPF_ENTRY_SW_CLUSTERS];
		uint CurClustersTotal = CurClustersHW + CurClustersSW;

		FVirtualShadowMapProjectionShaderData ProjectionData = DecodeVirtualShadowMapProjectionData(PrevProjectionData, FVirtualShadowMapHandle::MakeFromId(BufferIndex));

		GPUMessageWriteItem(Mw, ProjectionData.LightId);
		GPUMessageWriteItem(Mw, CurClustersTotal);
	}
	for (uint Index = 0; Index < MAX_NPF_DIAGNOSTIC_SLOTS_MULTIPAGE; ++Index)
	{
		uint BufferIndex = VSM_MAX_SINGLE_PAGE_SHADOW_MAPS + Index;

		uint CurClustersHW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * BufferIndex + VSM_NPF_ENTRY_HW_CLUSTERS];
		uint CurClustersSW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * BufferIndex + VSM_NPF_ENTRY_SW_CLUSTERS];
		uint CurClustersTotal = CurClustersHW + CurClustersSW;

		FVirtualShadowMapProjectionShaderData ProjectionData = DecodeVirtualShadowMapProjectionData(PrevProjectionData, FVirtualShadowMapHandle::MakeFromId(BufferIndex));

		GPUMessageWriteItem(Mw, ProjectionData.LightId);
		GPUMessageWriteItem(Mw, CurClustersTotal);
	}
}

#define NANITE_GEO_HISTOGRAM_NUM_BINS (VSM_GENERATE_STATS_GROUPSIZE*2)

groupshared uint NaniteGeoHistogramHWClusters[NANITE_GEO_HISTOGRAM_NUM_BINS];
groupshared uint NaniteGeoHistogramSWClusters[NANITE_GEO_HISTOGRAM_NUM_BINS];
groupshared uint NaniteGeoHistogramMax;
groupshared uint NaniteGeoHistogramNonZeroBins;

void BuildNaniteGeoHistogram(uint ThreadId)
{
	if (ThreadId == 0)
	{
		NaniteGeoHistogramMax = 0;
		NaniteGeoHistogramNonZeroBins = 0;
	}

	GroupMemoryBarrierWithGroupSync();

	// Histogram with N bins with rollover: if #VSMs > N, the stats for VSM i are accumulated in bin (i % N).
	// Single page SMs are treated separately.

	NaniteGeoHistogramHWClusters[ThreadId] = 0;
	NaniteGeoHistogramSWClusters[ThreadId] = 0;
	for (int Index = ThreadId; Index < VSM_MAX_SINGLE_PAGE_SHADOW_MAPS; Index += VSM_GENERATE_STATS_GROUPSIZE)
	{
		uint CurClustersHW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * Index + VSM_NPF_ENTRY_HW_CLUSTERS];
		uint CurClustersSW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * Index + VSM_NPF_ENTRY_SW_CLUSTERS];
		uint CurClustersTotal = CurClustersHW + CurClustersSW;
		
		NaniteGeoHistogramHWClusters[ThreadId] += CurClustersHW;
		NaniteGeoHistogramSWClusters[ThreadId] += CurClustersSW;
		InterlockedMax(NaniteGeoHistogramMax, CurClustersTotal);
	}

	NaniteGeoHistogramHWClusters[VSM_GENERATE_STATS_GROUPSIZE+ThreadId] = 0;
	NaniteGeoHistogramSWClusters[VSM_GENERATE_STATS_GROUPSIZE+ThreadId] = 0;
	for (int Index = VSM_MAX_SINGLE_PAGE_SHADOW_MAPS + ThreadId; Index < NanitePerformanceFeedbackNumEntries; Index += VSM_GENERATE_STATS_GROUPSIZE)
	{
		uint CurClustersHW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * Index + VSM_NPF_ENTRY_HW_CLUSTERS];
		uint CurClustersSW = NanitePerformanceFeedback[VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * Index + VSM_NPF_ENTRY_SW_CLUSTERS];
		uint CurClustersTotal = CurClustersHW + CurClustersSW;
		
		NaniteGeoHistogramHWClusters[VSM_GENERATE_STATS_GROUPSIZE+ThreadId] += CurClustersHW;
		NaniteGeoHistogramSWClusters[VSM_GENERATE_STATS_GROUPSIZE+ThreadId] += CurClustersSW;
		InterlockedMax(NaniteGeoHistogramMax, CurClustersTotal);
	}

	for (int Index = ThreadId; Index < NANITE_GEO_HISTOGRAM_NUM_BINS; Index += VSM_GENERATE_STATS_GROUPSIZE)
	{
		if (NaniteGeoHistogramHWClusters[Index] + NaniteGeoHistogramSWClusters[Index] > 0)
		{
			InterlockedAdd(NaniteGeoHistogramNonZeroBins, 1);
		}
	}

	GroupMemoryBarrierWithGroupSync();
}

void PrintStats()
{
	const bool bShowBasic              = (ShowStatsValue & EVSMStatSection::Basic) > 0;
	const bool bShowPhysicalPages      = (ShowStatsValue & EVSMStatSection::PhysicalPages) > 0;
	const bool bShowNonNaniteInstances = (ShowStatsValue & EVSMStatSection::NonNanite) > 0;
	const bool bShowNaniteClusters     = (ShowStatsValue & EVSMStatSection::Clusters) > 0;

	const float TopMargin = 0.05f;
	const float HeadlineX = 0.47f;
	const float ItemX = 0.48f;
	FShaderPrintContext Context = InitShaderPrintContext(true, float2(HeadlineX, TopMargin));
	const int NumLabelCharacters = 26;
	const float ValueX = HeadlineX + Context.Config.FontSpacing.x * NumLabelCharacters;

	if (bShowBasic || bShowPhysicalPages)
	{
		Context.Pos.x = HeadlineX;
		Print(Context, TEXT("Physical Pages"), FontWhite);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, VirtualShadowMap.MaxPhysicalPages);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Requested"), FontWhite);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_REQUESTED_THIS_FRAME_PAGES]);
		Newline(Context);

		if (bShowPhysicalPages)
		{
			Context.Pos.x = ItemX;
			Print(Context, TEXT("Empty"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_EMPTY_PAGES]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("Allocated New"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_ALLOCATED_NEW]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("Cleared"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NUM_PAGES_TO_CLEAR]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("HZB Built"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NUM_HZB_PAGES_BUILT]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("Prfiltered Distant"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_PREFILTERED_DISTANT_PAGES]);
			Newline(Context);
		}

		{
			Context.Pos.x = HeadlineX;
			Print(Context, TEXT("Static Cached"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_STATIC_CACHED_PAGES]);
			Newline(Context);

			if (bShowPhysicalPages)
			{
			Context.Pos.x = ItemX;
			Print(Context, TEXT("Invalidated"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_STATIC_INVALIDATED_PAGES]);
			Newline(Context);
			}

			Context.Pos.x = HeadlineX;
			Print(Context, TEXT("Dynamic Cached"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_DYNAMIC_CACHED_PAGES]);
			Newline(Context);

			if (bShowPhysicalPages)
			{
			Context.Pos.x = ItemX;
			Print(Context, TEXT("Invalidated"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_DYNAMIC_INVALIDATED_PAGES]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("Considered for WPO"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_WPO_CONSIDERED_PAGES]);
			Newline(Context);

			Context.Pos.x = ItemX;
			Print(Context, TEXT("Merged"), FontWhite);
			Context.Pos.x = ValueX;
			PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NUM_PAGES_TO_MERGE]);
			Newline(Context);
		}
		}

		Context.Pos.x = HeadlineX;
		Print(Context, TEXT("Global Resolution Bias   "), FontWhite);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, VirtualShadowMap.GlobalResolutionLodBias);
		Newline(Context);
	}

	float GlobalComputeThrottleIntensity = 0.0f;
	if (bThrottlingEnabled)
	{
		GlobalComputeThrottleIntensity = asfloat(ThrottleBuffer[VSM_TB_HEADER_TOTAL_THROTTLE]);
	}

	Context.Pos.x = HeadlineX;
	Print(Context, TEXT("Compute Throttle"), FontWhite);
	Context.Pos *= float2(Context.Config.Resolution);
	float ValueWidth = Context.Config.FontSpacing.x * 9.5;
	float BarStartX = ValueX * Context.Config.Resolution.x;
	float BarEndX = (ValueX + ValueWidth) * Context.Config.Resolution.x;
	float BarHeight = 8.0f;
	AddFilledQuadSS(Context, float2(BarStartX, Context.Pos.y - BarHeight/2), float2(BarEndX, Context.Pos.y + BarHeight/2), float4(0,0,0,0.5));
	AddFilledQuadSS(Context, float2(BarStartX, Context.Pos.y - BarHeight/2), float2(lerp(BarStartX, BarEndX, GlobalComputeThrottleIntensity), Context.Pos.y + BarHeight/2), float4(1,0,0,1.0));
	Context.Pos /= float2(Context.Config.Resolution);
	Newline(Context);

	if (bShowNonNaniteInstances)
	{
		Context.Pos.x = HeadlineX;
		Print(Context, TEXT("Non-Nanite Instances"), FontCyan);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Total"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_TOTAL]);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Drawn"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_DRAWN]);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("HZB Culled"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_HZB_CULLED]);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Page Mask Culled"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_PAGE_MASK_CULLED]);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Empty Rect Culled"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_EMPTY_RECT_CULLED]);
		Newline(Context);

		Context.Pos.x = ItemX;
		Print(Context, TEXT("Frustum Culled"), FontCyan);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, InStatsBuffer[VSM_STAT_NON_NANITE_INSTANCES_FRUSTUM_CULLED]);
		Newline(Context);
	}

	uint ClustersHW = NanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_HW_CLUSTERS];
	uint ClustersSW = NanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_SW_CLUSTERS];

	if (bShowBasic && !bShowNaniteClusters)
	{
	Context.Pos.x = HeadlineX;
		Print(Context, TEXT("Nanite Clusters"), FontWhite);
		Context.Pos.x = ValueX;
		PrintLeftAlign(Context, ClustersHW + ClustersSW);
		Newline(Context);
	}
	else if (bShowNaniteClusters)
	{
		Context.Pos.x = HeadlineX;
	Print(Context, TEXT("Nanite Clusters HW"), FontWhite);
	Context.Pos.x = ValueX;
	PrintLeftAlign(Context, ClustersHW);
	Newline(Context);

	Context.Pos.x = HeadlineX;
	Print(Context, TEXT("Nanite Clusters SW"), FontWhite);
	Context.Pos.x = ValueX;
	PrintLeftAlign(Context, ClustersSW);
	Newline(Context);

	Context.Pos.x = HeadlineX;
	Print(Context, TEXT("Nanite Clusters Per VSM"), FontWhite);
	Newline(Context);
	
	float2 HistogramScale = float2(4, 200);
	uint HistogramMax = max(4096U, 2U << firstbithigh(NaniteGeoHistogramMax));
	float4 HistogramMaxIndicatorColor = float4(GreenToRedTurbo((firstbithigh(HistogramMax) - 12) / 4.0f), 1.0); // green to red, 4096 to 32768
	if (NaniteGeoHistogramNonZeroBins < 64)
	{
		HistogramScale.x = 8;
	}

	Print(Context, HistogramMax, FontGrey);
	Context.Pos.x = HeadlineX;
	Context.Pos.y += HistogramScale.y / Context.Config.Resolution.y;
	Print(Context, 0U, FontGrey);
	Context.Pos.x = HeadlineX;
	Context.Pos.y -= HistogramScale.y / Context.Config.Resolution.y;

	Context.Pos *= float2(Context.Config.Resolution);
	float4 HistogramRect = float4(
		Context.Pos,
		Context.Pos + float2(HistogramScale.x * max(50, NaniteGeoHistogramNonZeroBins), HistogramScale.y)
	);
	AddFilledQuadSS(Context, HistogramRect.xy, HistogramRect.zw, float4(HistogramMaxIndicatorColor.rgb * 0.25,0.4));
	Context.Pos.y += HistogramScale.y;
	for (int Index = 0; Index < NANITE_GEO_HISTOGRAM_NUM_BINS; Index++)
	{
		float HWClusters = (NaniteGeoHistogramHWClusters[Index] / float(HistogramMax)) * HistogramScale.y;
		float SWClusters = (NaniteGeoHistogramSWClusters[Index] / float(HistogramMax)) * HistogramScale.y;
		
		bool bIsSinglePage = Index < VSM_GENERATE_STATS_GROUPSIZE;
		FVirtualShadowMapHandle VSMHandle;
		if (bIsSinglePage)
		{
			VSMHandle = FVirtualShadowMapHandle::MakeFromId(Index);
		}
		else
		{
			VSMHandle = FVirtualShadowMapHandle::MakeFromFullIndex(Index - VSM_GENERATE_STATS_GROUPSIZE);
		}
		float ThrottleIntensity = 0.0f;
		if (bThrottlingEnabled)
		{
			ThrottleIntensity = asfloat(ThrottleBuffer[VSM_TB_SIZEOF_HEADER + VSM_TB_SIZEOF_ENTRY * VSMHandle.GetDataIndex() + VSM_TB_ENTRY_THROTTLE]);
		}

		bool bHasClusterData = HWClusters > 0 || SWClusters > 0;

		if (HWClusters > 0) { AddFilledQuadSS(Context, Context.Pos, Context.Pos + float2(HistogramScale.x, -HWClusters), float4(0,0,1,1.0)); }
		if (SWClusters > 0) { AddFilledQuadSS(Context, Context.Pos + float2(0, -HWClusters), Context.Pos + float2(0, -HWClusters) + float2(HistogramScale.x, -SWClusters), float4(0,1,1,1.0)); }
		if (bHasClusterData && ThrottleIntensity > 0) { AddFilledQuadSS(Context, Context.Pos, Context.Pos + float2(HistogramScale.x, ThrottleIntensity * HistogramScale.y), float4(1,0,0,1.0)); }
		if (bHasClusterData) { Context.Pos.x += HistogramScale.x; }
	}
	Context.Pos /= float2(Context.Config.Resolution);
	Newline(Context);
}
}



#if VSM_GENERATE_STATS
[numthreads(VSM_GENERATE_STATS_GROUPSIZE, 1, 1)]
#else
[numthreads(1, 1, 1)]
#endif
void LogVirtualSmStatsCS(uint ThreadId : SV_DispatchThreadID)
{
	if (ThreadId == 0 && StatusMessageId != INDEX_NONE)
	{
		SendStatusMessage();
	}
	
#if VSM_GENERATE_STATS
	if (ThreadId == 0 && StatsMessageId != INDEX_NONE)
	{
		SendStatsMessage();
	}

	if (ShowStatsValue > 0U)
	{
		BuildNaniteGeoHistogram(ThreadId);

		if (ThreadId == 0)
		{
			PrintStats();
		}
	}
#endif
}

uint ShadowMapIdRangeStart;
uint ShadowMapIdRangeEnd;

[numthreads(1, 1, 1)]
void PrintClipmapStats()
{
	float TopMargin = 0.35f;
	float HeadlineX = 0.07f;
	float ItemX = 0.08f;

	FShaderPrintContext Context = InitShaderPrintContext(true, float2(HeadlineX, TopMargin));

	// Main Pass
	Context.Pos.x = ItemX;
	Print(Context, TEXT("Allocated Page Rects"));
	Newline(Context);
	uint TotalArea = 0U;
	{
		for (uint SmId = ShadowMapIdRangeStart; SmId < ShadowMapIdRangeEnd; ++SmId)
		{
			uint4 Rect = AllocatedPageRectBounds[SmId * VSM_MAX_MIP_LEVELS];
			Context.Pos.x = ItemX;
			Print(Context, Rect.x);//ShaderPrintLeftAlign(ScreenPos, Rect.x);
			PrintSymbol(Context, _SPC_);
			Print(Context, Rect.y);//ShaderPrintLeftAlign(ScreenPos, Rect.y);
			PrintSymbol(Context, _SPC_);
			Print(Context, Rect.z);//ShaderPrintLeftAlign(ScreenPos, Rect.z);
			PrintSymbol(Context, _SPC_);
			Print(Context, Rect.w);//ShaderPrintLeftAlign(ScreenPos, Rect.w);
			if (Rect.x <= Rect.z && Rect.y <= Rect.w)
			{
				PrintSymbol(Context, _SPC_);
				Print(Context, Rect.z - Rect.x + 1);//ShaderPrintLeftAlign(ScreenPos, Rect.z);
				PrintSymbol(Context, _SPC_);
				Print(Context, Rect.w - Rect.y + 1);//ShaderPrintLeftAlign(ScreenPos, Rect.w);

				uint Area = (Rect.z - Rect.x + 1) * (Rect.w - Rect.y + 1);
				PrintSymbol(Context, _SPC_);
				Print(Context, Area);//ShaderPrintLeftAlign(ScreenPos, Rect.w);

				TotalArea += Area;
			}
			Newline(Context);
		}
	}
	Context.Pos.x = ItemX;
	Print(Context, TEXT("Total Area "));
	Print(Context, TotalArea);//ShaderPrintLeftAlign(ScreenPos, Rect.x);
	Newline(Context);
}
