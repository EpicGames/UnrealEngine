// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define VSM_PLATFORM_PROJECTION_SHADER 1
#ifdef OVERRIDE_VSM_PLATFORM_CONFIG_USH
#include "/Platform/Private/VSMPlatformConfig.ush"
#endif

#include "../SceneTexturesCommon.ush"
#include "../Substrate/Substrate.ush"
#include "../DeferredShadingCommon.ush"
#include "../DynamicLightingCommon.ush"
#include "../MortonCode.ush"
#include "../LightShaderParameters.ush"
#include "../LightGridCommon.ush"
#include "../HairStrands/HairStrandsVisibilityCommon.ush"
#include "../Visualization.ush"
#include "../BlueNoise.ush"

int SMRTSamplePerRay;
uint bSMRTExtrapolateSlope;

#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapProjectionFilter.ush"
#include "VirtualShadowMapProjectionDirectional.ush"
#include "VirtualShadowMapProjectionSpot.ush"
#include "VirtualShadowMapTransmissionCommon.ush"
#include "VirtualShadowMapScreenRayTrace.ush"
#include "VirtualShadowMapLightGrid.ush"
#include "VirtualShadowMapStats.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"

#if HAS_HAIR_STRANDS
#define VOXEL_TRAVERSAL_TYPE VOXEL_TRAVERSAL_LINEAR_MIPMAP
#define VOXEL_TRAVERSAL_DEBUG 0
#include "../HairStrands/HairStrandsVoxelPageCommon.ush"
#include "../HairStrands/HairStrandsVoxelPageTraversal.ush"
#if USE_BLUE_NOISE
DEFINE_HAIR_BLUE_NOISE_JITTER_FUNCTION
#endif
#endif

#ifndef DIRECTIONAL_LIGHT
#define DIRECTIONAL_LIGHT 0
#endif

#ifndef VISUALIZE_OUTPUT
#define VISUALIZE_OUTPUT 0
#endif

#define SCREEN_RAY_SAMPLES 4

StructuredBuffer< FPhysicalPageMetaData > PhysicalPageMetaData;

struct FProjectionShadingInfo
{
	bool bIsValid;
	bool bIsHair;
	bool bIsSubsurface;
	bool bDisableBackFaceCulling;
	bool bIsFirstPerson;

	float3 WorldNormal;
};

FProjectionShadingInfo GetProjectionShadingInfo(uint2 PixelPos)
{
	FProjectionShadingInfo Out;
	Out.bDisableBackFaceCulling = false;

#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(PixelPos, 0)));

	const bool bHasSSS = SubstratePixelHeader.HasSubsurface();
	const uint BSDFType = SubstratePixelHeader.SubstrateGetBSDFType();

	Out.bIsValid = SubstratePixelHeader.IsSubstrateMaterial();
	Out.bIsHair = BSDFType == SUBSTRATE_BSDF_TYPE_HAIR;
	Out.WorldNormal = TopLayerData.WorldNormal;
	Out.bIsSubsurface = bHasSSS; // legacy IsSubsurfaceModel is true for subsurface profile, eyes, etc.

	// VSM traces using the top layer normal when NoL>0 only. So if normals for slabs below do not match, light leaking will occur.
	// For instance: a flat translucent top layer above a normal mapped bottom layer.
	// That is why, for materials with multiple local bases, we must not do backface culling.
	if (SubstratePixelHeader.GetLocalBasesCount() > 1)
	{
		Out.bDisableBackFaceCulling = true;
	}
	Out.bIsFirstPerson = SubstratePixelHeader.IsFirstPerson();
#else
	const FGBufferData GBufferData = GetGBufferDataUint(PixelPos, true);
	Out.bIsValid = GBufferData.ShadingModelID != SHADINGMODELID_UNLIT;
	Out.bIsHair = GBufferData.ShadingModelID == SHADINGMODELID_HAIR;
	Out.bIsSubsurface = IsSubsurfaceModel(GBufferData.ShadingModelID);
	Out.bDisableBackFaceCulling = GBufferData.ShadingModelID == SHADINGMODELID_CLEAR_COAT;
	Out.WorldNormal = GBufferData.WorldNormal;
	Out.bIsFirstPerson = IsFirstPerson(GBufferData);
#endif

	Out.bDisableBackFaceCulling = Out.bDisableBackFaceCulling || Out.bIsSubsurface || Out.bIsHair;

	return Out;
}

int GetSMRTRayCount()
{
#if DIRECTIONAL_LIGHT
	return VirtualShadowMap.SMRTRayCountDirectional;
#else
	return VirtualShadowMap.SMRTRayCountLocal;
#endif
}

SCHEDULER_MIN_PRESSURE
MAX_OCCUPANCY

// NOTE: Not currently used by occasionally useful for testing
float3 GetEstimatedGeoWorldNormal(float3 TranslatedWorldPosition, float3 ShadingNormal)
{
	// Figure out slope, we do world space since that is the space where we might override using the shading normal...
	float3 TranslatedWorldPositionDDX = ddx_fine(TranslatedWorldPosition);
	float3 TranslatedWorldPositionDDY = ddy_fine(TranslatedWorldPosition);
	float3 EstimatedGeoWorldNormal = cross(TranslatedWorldPositionDDX, TranslatedWorldPositionDDY);

	// Handle NaNs; will cause it to revert to shading normal below
	float LengthSq = dot(EstimatedGeoWorldNormal, EstimatedGeoWorldNormal);
	EstimatedGeoWorldNormal = LengthSq > 1e-8f ? normalize(EstimatedGeoWorldNormal) : float3(0, 0, 0);

#if 1
	// NOTE: Gbuffer normal is not the surface normal for hair; hair lighting takes a different path but possibly
	// necessary to do some sort of special casing here (disable normal offset and biasing entirely?).
	// If the estimated geo normal is too far out we assume it's broken (derivative includes other surfaces or background) and fall back to the shading normal
	if (dot(ShadingNormal, EstimatedGeoWorldNormal) < 0.25f)
	{
		EstimatedGeoWorldNormal = ShadingNormal;
	}
#endif

	return EstimatedGeoWorldNormal;
}

float4 ComputeRandom4(uint2 PixelPosition)
{
	const uint InSeed = View.StateFrameIndexMod8;
	const uint2 Seed0 = Rand3DPCG16(int3(PixelPosition, InSeed)).xy;
	const uint2 Seed1 = Rand3DPCG16(int3(PixelPosition + 17, InSeed)).xy;
	return float4(
		Hammersley16(InSeed, 8, Seed0),
		Hammersley16(InSeed, 8, Seed1));
}


#if PREFILTERED_DISTANT_CLIPMAP

Texture2D<float> PrefilteredDistantPhysicalPagePool;

float SamplShadowMaskMapVirtualBilinear(FVirtualShadowMapHandle VirtualShadowMapHandle, float2 VirtualTexelAddressFloat)
{
	FVirtualShadowMapFilterableContext Context;
	Context.PrefilteredDistantPhysicalPagePool = PrefilteredDistantPhysicalPagePool;
	Context.PageTable = VirtualShadowMap.PageTable;

	FVirtualSMLevelOffset PageTableLevelOffset = CalcPageTableLevelOffset(VirtualShadowMapHandle, 0u);

	float Result = 0.0f;
	Context.SampleBilinear(PageTableLevelOffset, VirtualTexelAddressFloat, Result);
	return Result;
}

FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectionalFilterableMapped(FVirtualShadowMapHandle MappedClipmapHandle, float3 TranslatedWorldPositionView)
{
	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(MappedClipmapHandle);
	float2 VirtualTexelAddressFloat = ViewTranslatedWorldToVirtualTexel(ProjectionData, TranslatedWorldPositionView);
	uint2 VirtualTexelAddress = uint2(floor(VirtualTexelAddressFloat));
	uint2 BasePage = VirtualTexelAddress >> VSM_LOG2_PAGE_SIZE;

	FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(MappedClipmapHandle, 0, BasePage));		
	{

		Result.VirtualTexelAddress = VirtualTexelAddress;
		Result.PhysicalTexelAddress = PhysicalPageEntry.PhysicalAddress * VSM_PAGE_SIZE + (Result.VirtualTexelAddress & VSM_PAGE_SIZE_MASK);

		Result.ShadowFactor = SamplShadowMaskMapVirtualBilinear(MappedClipmapHandle, VirtualTexelAddressFloat);

		Result.bValid = true;
		Result.ClipmapOrMipLevel = ProjectionData.ClipmapLevel;

		// Note: a constant is more stable than varying, maybe we need to store a 8-bit depth to get something that works well.
		Result.OccluderDistance = ComputeOccluderDistanceOrtho(
			ProjectionData.ShadowViewToClipMatrix,
			ProjectionData.DynamicDepthCullRange.y,
			ProjectionData.DynamicDepthCullRange.x);
	}

	return Result;
}

FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectionalPrefilteredDistant(
	int VirtualShadowMapId,
	FLightShaderParameters Light,
	uint2 PixelPos,
	float3 TranslatedWorldPosition)
{
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(VirtualShadowMapId);

	FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	checkSlow(BaseProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL);
	
 	// Scalarize? WaveReadLaneFirst(
	const float ClipmapLevelFloat = CalcBiasedAbsoluteClipmapLevelForSampling(BaseProjectionData, TranslatedWorldPosition, -1.0f);
	const int ClipmapLevel = int(floor(ClipmapLevelFloat));
	int ClipmapIndexEnd = max(0, ClipmapLevel - BaseProjectionData.ClipmapLevel);

	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	int ClipmapIndex = ClipmapIndexEnd + 1;
	if (ClipmapIndex < BaseProjectionData.ClipmapLevelCountRemaining)
	{
		FVirtualShadowMapHandle ClipmapLevelVirtualShadowMapHandle = VirtualShadowMapHandle.MakeOffset(ClipmapIndex);

		FVirtualShadowMapSampleResult LevelResult = SampleVirtualShadowMapDirectionalFilterableMapped(ClipmapLevelVirtualShadowMapHandle, TranslatedWorldPosition);
		if (LevelResult.bValid)
		{
			Result.ClipmapOrMipLevel = LevelResult.ClipmapOrMipLevel;
			Result.RayCount = LevelResult.RayCount;
			Result.VirtualTexelAddress = LevelResult.VirtualTexelAddress;
			Result.PhysicalTexelAddress = LevelResult.PhysicalTexelAddress;
			Result.GeneralDebug = LevelResult.GeneralDebug;

			Result.ShadowFactor *= LevelResult.ShadowFactor;
			Result.bValid = true;
			Result.OccluderDistance = LevelResult.OccluderDistance;

			// early out on full occlusion
			if (Result.ShadowFactor < 0.00000001f)
			{
				Result.ShadowFactor = 0.0f;
			}
		}
	}
	return Result;
}

#endif

uint bPrefilteredDistantClipmap;
uint bFirstPersonShadow;
uint bOnePassProjection;
uint bVisualizeOutput;
int4 ProjectionRect;
float SubsurfaceMinSourceRadius;

// One pass projection
#if ONE_PASS_PROJECTION || UBERSHADER
RWTexture2D< uint4 > OutShadowMaskBits;
#endif
#if !ONE_PASS_PROJECTION || UBERSHADER
// Single light per pass
// Light parameters loaded via GetRootLightShaderParameters();
int LightUniformVirtualShadowMapId;
RWTexture2D< float2 > OutShadowFactor;
#endif

// Visualization output
RWTexture2D< float4 > OutVisualize;
int VisualizeModeId;
int bVisualizeCachedPagesOnly;
int VisualizeVirtualShadowMapId;

#if FIRST_PERSON_SHADOW
int FirstPersonVirtualShadowMapId;
#endif

// Type of input data consume by the page allocation (i.e., data read from the source buffer: Gbuffer, HairStrands data, ...)
#define INPUT_TYPE_GBUFFER 0
#define INPUT_TYPE_HAIRSTRANDS 1
uint InputType;
uint bCullBackfacingPixels;

FVirtualShadowMapSampleResult ProjectLight(
	int VirtualShadowMapId,
	FLightShaderParameters Light,
	FProjectionShadingInfo ShadingInfo,
	uint2 PixelPos,
	float SceneDepth,
	float ScreenRayLengthWorld,
	float3 TranslatedWorldPosition,
	const float Noise,
	const FSubsurfaceOpacityMFP SubsurfaceOpacityMFP)
{
	const bool bIsHairInput = InputType == INPUT_TYPE_HAIRSTRANDS;
	
	bool bBackfaceCull = (bCullBackfacingPixels > 0) && !bIsHairInput && !ShadingInfo.bDisableBackFaceCulling;
#if VISUALIZE
	// Modes more related to the shadow depth rendering don't want to see the projection culling
	bBackfaceCull = bBackfaceCull &&
		VisualizeModeId != VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW;
#endif

#if DIRECTIONAL_LIGHT
	float3 L = Light.Direction;
	bool bInLightRegion = true;

	if (bBackfaceCull && IsBackfaceToDirectionalLight(ShadingInfo.WorldNormal, Light.Direction, Light.SourceRadius))
	{
		bInLightRegion = false;
	}

#else	// Local light

	float3 ToLight = Light.TranslatedWorldPosition - TranslatedWorldPosition;
	float InvDist = rsqrt(dot(ToLight, ToLight));
	float3 L = ToLight * InvDist;
	
	bool bInLightRadius = InvDist >= Light.InvRadius;
	bool bInLightCone = SpotAttenuationMask(L, -Light.Direction, Light.SpotAngles) > 0.0f;

	bool bInLightRegion = bInLightRadius && bInLightCone;

	const bool bIsRectLight = Light.RectLightBarnLength > -2.0f;
	if (bIsRectLight)
	{
		const bool bIsBehindRectLight = dot(ToLight, Light.Direction) < 0;
		if (bIsBehindRectLight)
		{
			bInLightRegion = false;
		}
	}

	if (bInLightRegion && bBackfaceCull)
	{
		if (IsBackfaceToLocalLight(ToLight, ShadingInfo.WorldNormal, Light.SourceRadius))
		{
			bInLightRegion = false;
		}
	}

#endif // DIRECTIONAL_LIGHT
	
	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	const bool bValidPixel = bIsHairInput || ShadingInfo.bIsValid;

	BRANCH
	if (bInLightRegion && bValidPixel)
	{
		const float3 WorldNormal = (bIsHairInput || ShadingInfo.bIsHair) ? L : ShadingInfo.WorldNormal;

		if (!ShadingInfo.bIsSubsurface)
		{
			TranslatedWorldPosition += WorldNormal * VirtualShadowMapGetNormalBiasLength(TranslatedWorldPosition);
		}

		// Do not run contact shadow when computing the hair shadow mask (i.e. shadow mask applied on hair, has the scene 
		// depth buffer contains fully opaque depth, which create false positive intersection resulting in wrong self shadowing)
		float SMRTRayOffset = ScreenRayLengthWorld;
		if (!bIsHairInput && !ShadingInfo.bIsSubsurface)
		{
			if (ScreenRayLengthWorld > 0.0f)
			{
				// Trace a screen space ray to try and get "away" from the receiver surface before
				// we trace the SMRT ray to avoid mismatches/incorrect self-shadowing.
				SMRTRayOffset = VirtualShadowMapScreenRayCast(
					TranslatedWorldPosition,
					L,
					ScreenRayLengthWorld,
					Noise);
			}
		}

#if DIRECTIONAL_LIGHT

		bool bFullyShadowed = false;
		#if PREFILTERED_DISTANT_CLIPMAP
		IF_UBERSHADER(bPrefilteredDistantClipmap)
		{
			// Sample the perfiltered distant SM first and early out if that is already producing fully shadowed samples.
			// TODO: reuse the mapped clipmap search (create some intermediate struct that can be passed between them?)
			Result = SampleVirtualShadowMapDirectionalPrefilteredDistant(VirtualShadowMapId, Light, PixelPos, TranslatedWorldPosition);
			bFullyShadowed = Result.bValid && Result.ShadowFactor <= 0.0001f;
		}
		#endif // PREFILTERED_DISTANT_CLIPMAP

		if (!bFullyShadowed)
		{
			FVirtualShadowMapSampleResult SmrtResult = InitVirtualShadowMapSampleResult();

			if (GetSMRTRayCount() > 0)
			{
				SmrtResult = TraceDirectional(
					VirtualShadowMapId,
					Light,
					PixelPos,
					SceneDepth,
					TranslatedWorldPosition,
					SMRTRayOffset,
					Noise,
					WorldNormal);
			}
			else
			{
				SmrtResult = SampleVirtualShadowMapDirectional(
					VirtualShadowMapId,
					TranslatedWorldPosition,
					SMRTRayOffset,
					WorldNormal);
			}

			if (Result.bValid)
			{
				// SMRT has the peculiarity of reporting 0 occluder distance if nothing was hit, which seems incosistent
				if (SmrtResult.ShadowFactor < 1.0f)
				{
					// Note: combine the occlusion distance such that the weight is towards that which produced the most coverage. This prevents nearby occlusders that 
					// became visible from cauisng large spikes in transmission. Not 100% robust.
					Result.OccluderDistance = lerp(Result.OccluderDistance, SmrtResult.OccluderDistance, Result.ShadowFactor);
					// TODO: fairly arbitrary combination of 
					// float w0 = (1.0f - Result.ShadowFactor);
					// float w1 = (1.0f - SmrtResult.ShadowFactor);
					// Result.OccluderDistance = (Result.OccluderDistance * w0 + SmrtResult.OccluderDistance * w1) / (w0 + w1);
					// Result.OccluderDistance = min(Result.OccluderDistance, SmrtResult.OccluderDistance);
					// Result.OccluderDistance = SmrtResult.OccluderDistance;
				}
				Result.ShadowFactor = Result.ShadowFactor * SmrtResult.ShadowFactor;

			}
			else
			{
				Result = SmrtResult;
			}
		}

		#if FIRST_PERSON_SHADOW
		IF_UBERSHADER(bFirstPersonShadow)
		{
			// If there is an extra FP shadow map, it represents the shadow cast from the 3rd person mesh of the player onto the world only, thus it should only be sampled if
			// 1. the pixel is not marked as FP (e.g., the weapon)
			// 
			// 2. the pixel is not already in shadow (100%)
			bFullyShadowed = Result.bValid && Result.ShadowFactor == 0.0f;

			if (FirstPersonVirtualShadowMapId >= 0 && !ShadingInfo.bIsFirstPerson && !bFullyShadowed)
			{
				FVirtualShadowMapSampleResult FPResult = InitVirtualShadowMapSampleResult();
				if (GetSMRTRayCount() > 0)
				{
					FPResult = TraceDirectional(
						FirstPersonVirtualShadowMapId,
						Light,
						PixelPos,
						SceneDepth,
						TranslatedWorldPosition,
						SMRTRayOffset,
						Noise,
						WorldNormal);
				}
				else
				{
					FPResult = SampleVirtualShadowMapDirectional(
						FirstPersonVirtualShadowMapId,
						TranslatedWorldPosition,
						SMRTRayOffset,
						WorldNormal,
						SceneDepth);
				}
				if (FPResult.bValid)
				{
					if (!Result.bValid)
					{
						Result = FPResult;
					}
					else
					{
						// Merge the result, picking the most-shadowed value and the associated occluder distance
						Result.ShadowFactor = min(Result.ShadowFactor, FPResult.ShadowFactor);
						Result.OccluderDistance = Result.ShadowFactor < FPResult.ShadowFactor ? Result.OccluderDistance : FPResult.OccluderDistance;
					}
				}
			}
		}
		#endif // FIRST_PERSON_SHADOW


#else // !DIRECTIONAL_LIGHT
		if (GetSMRTRayCount() > 0)
		{
			Result = TraceLocalLight(
				VirtualShadowMapId,
				Light,
				PixelPos,
				SceneDepth,
				TranslatedWorldPosition,
				SMRTRayOffset,
				Noise,
				WorldNormal);
		}
		else
		{
			Result = SampleVirtualShadowMapLocal(
				VirtualShadowMapId,
				TranslatedWorldPosition,
				SMRTRayOffset,
				WorldNormal,
				true, // OptimalBias
				SceneDepth
			);
		}
#endif // DIRECTIONAL_LIGHT
		
		// Explanation:
		//	- If bDataIsOpacity = 1: Data stores the surface's SSS opacity. We compute a transmitted color when the surface is not fully opaque (opacity == 1).
		//	- If bDataIsOpacity = 0: Data stores the surface's MFP.         We compute a transmitted color when the surface is not fully opaque (MFP == 0)
		if ((SubsurfaceOpacityMFP.bDataIsOpacity && SubsurfaceOpacityMFP.Data < 1.0f) || (!SubsurfaceOpacityMFP.bDataIsOpacity && SubsurfaceOpacityMFP.Data > 0.0f))
		{
			// We use a single term for both the shadow and "transmission shadow" in the VSM path to provide a
			// higher quality filtered result on back faces and avoid artifacts at the point where normals flip.
			// This is especially important for foliage as normals are often "warped" to provide softer shading
			// and do not represent the underlying geometry accurately, so having a continuous shadow function
			// is important.
			Result.ShadowFactor = GetShadowFactorSubsurface(Result.ShadowFactor, Result.OccluderDistance, SubsurfaceOpacityMFP);
		}

		//Result.GeneralDebug = GreenToRedTurbo(1.0f - (SMRTRayOffset / ScreenRayLengthWorld));
		//Result.GeneralDebug = SpotAttenuation(-L, Light.Direction, Light.SpotAngles).xxx;
		//Result.GeneralDebug = (Result.OccluderDistance / 20.0f).xxx;

		// Hair strands voxel traversal to apply hair shadow on top of opaque geometry
		#if HAS_HAIR_STRANDS
		if (!bIsHairInput && Result.ShadowFactor > 0)
		{
			float AccShadowFactor = 0;
			// TODO: Maybe do this min() in the uniform buffer parameter itself?
			uint HairRayCount = max(1u, min(VirtualShadowMap.SMRTHairRayCount, uint(GetSMRTRayCount())));
			uint RayIndex = 0; 
			for (; RayIndex < HairRayCount; RayIndex++ )
			{
				float3 RayStart = 0;
				float3 RayEnd = 0;

				#if USE_BLUE_NOISE
				const float4 Random = GetHairBlueNoiseJitter(PixelPos, RayIndex, HairRayCount, View.StateFrameIndexMod8);
				#else
				const float4 Random = ComputeRandom4(PixelPos);
				#endif

			#if DIRECTIONAL_LIGHT
				const bool bCastHairRay = GenerateRayDirectional(
					Light,
					PixelPos,
					TranslatedWorldPosition,
					VirtualShadowMap.SMRTRayLengthScale * 100,
					RayIndex,
					HairRayCount,
					RayStart,
					RayEnd);
			#else
				const bool bCastHairRay = GenerateRayLocalLight(
					Light,
					PixelPos,
					TranslatedWorldPosition,
					WorldNormal,
					RayIndex,
					HairRayCount,
					RayStart,
					RayEnd);
			#endif

				if (bCastHairRay)
				{
					// Jitter start position to mask/compensate coarse voxel resolution
					float3 NormalizedDepthBias = 0;
					{
						const float PositionBiasScale = 0.5f;
						NormalizedDepthBias = (VirtualVoxel.DepthBiasScale_Shadow * L + PositionBiasScale * (Random.xyz * 2 - 1));
					}

					const float DistanceThreshold = 100000.0f;
					const float CoverageThreshold = 0.995f; // When Coverage is high, we do not trace shadow on opaque since hair/fur is covering the background.

					FVirtualVoxelCommonDesc CommonDesc;
					CommonDesc.PageCountResolution		= VirtualVoxel.PageCountResolution;
					CommonDesc.PageTextureResolution	= VirtualVoxel.PageTextureResolution;
					CommonDesc.PageResolution			= VirtualVoxel.PageResolution;
					CommonDesc.PageResolutionLog2		= VirtualVoxel.PageResolutionLog2;

					FHairTraversalSettings TraversalSettings = InitHairTraversalSettings();
					TraversalSettings.DensityScale		= VirtualVoxel.DensityScale_Shadow;
					TraversalSettings.CountThreshold	= 1;
					TraversalSettings.DistanceThreshold = DistanceThreshold;
					TraversalSettings.bDebugEnabled		= false;
					TraversalSettings.SteppingScale		= VirtualVoxel.SteppingScale_Shadow;
					TraversalSettings.Random			= Random.xyz;
					TraversalSettings.PixelRadius		= ConvertGivenDepthRadiusForProjectionType(VirtualVoxel.HairCoveragePixelRadiusAtDepth1, SceneDepth);
					TraversalSettings.bUseOpaqueVisibility = false;
					TraversalSettings.bCastShadow		= true;

					float TraversalHairCount = 0;
					const uint VoxelDescCount = VirtualVoxel.NodeDescCount;
					for (uint VoxelDescIt=0; TraversalHairCount < 1 && VoxelDescIt<VoxelDescCount; ++VoxelDescIt)
					{
						const FVirtualVoxelNodeDesc NodeDesc = UnpackVoxelNode(VirtualVoxel.NodeDescBuffer[VoxelDescIt], VirtualVoxel.PageResolution);

						FHairTraversalResult HairResult = InitHairTraversalResult();
						HairResult = ComputeHairCountVirtualVoxel(
							RayStart + NodeDesc.VoxelWorldSize * NormalizedDepthBias,
							RayEnd,
							CommonDesc,
							NodeDesc,
							VirtualVoxel.PageIndexBuffer,
							VirtualVoxel.PageTexture,
							TraversalSettings);
						TraversalHairCount += HairResult.HairCount;
					}
					const float IterationShadowFactor = saturate(1 - TraversalHairCount);
					AccShadowFactor += IterationShadowFactor;

					#define USE_ADAPTATIVE 1
					#if COMPUTESHADER && USE_ADAPTATIVE
					if (VirtualShadowMap.SMRTAdaptiveRayCount > 0)
					{
						// TODO: Adapt this heuristic based on SMRTAdaptiveRayCount as well?
						if( RayIndex == 0 )
						{
							bool bMiss = IterationShadowFactor >= 1.f;

							// All lanes missed
							bool bAllLanesMiss = WaveActiveAllTrue( bMiss );
							if( bAllLanesMiss )
							{
								break;
							}
						}
						else if( RayIndex >= VirtualShadowMap.SMRTAdaptiveRayCount )
						{
							bool bHit = IterationShadowFactor == 0;

							// After N iterations and all have hit, assume umbra
							bool bAllLanesHit = WaveActiveAllTrue( bHit );
							if( bAllLanesHit )
							{
								break;
							}
						}
					}
					#endif
				}
				else
				{
					AccShadowFactor += 1.f;
				}
			}

			AccShadowFactor /= float(max(1, min(RayIndex, HairRayCount)));
			Result.ShadowFactor = min(Result.ShadowFactor, AccShadowFactor);
		}
		#endif
	}

	return Result;
}

float3 GetVirtualPageColor(uint2 VirtualTexelAddress)
{
	uint2 vPage = VirtualTexelAddress.xy >> VSM_LOG2_PAGE_SIZE;
	return IntToColor(vPage.x + (vPage.y << 10U));
}

float Grayscale(float3 SourceClr)
{
	return 0.299 * SourceClr.r + 0.587 * SourceClr.g + 0.114 * SourceClr.b;
}

void OutputVisualize(
	int VirtualShadowMapId,
	uint2 PixelPos,
	float3 TranslatedWorldPosition,	// Unmodified, raw position before any biasing
	FVirtualShadowMapSampleResult Result)
{
	bool bOneLightVisible = VisualizeVirtualShadowMapId >= 0;
	if (bOneLightVisible && VisualizeVirtualShadowMapId != VirtualShadowMapId)
	{
		return;
	}

	// NOTE: Important to *not* write output if it isn't a recognized mode, as a different
	// pass may write that output instead.
	float4 Output = float4( 1, 0, 1, 0 );

	if ( Result.bValid )
	{
		const float3 vPageColor = GetVirtualPageColor(Result.VirtualTexelAddress);

		uint2 pPage = Result.PhysicalTexelAddress.xy >> VSM_LOG2_PAGE_SIZE;
		bool bValidPageAddress = all( pPage < uint2(VirtualShadowMap.PhysicalPoolSizePages) );
		
		FPhysicalPageMetaData pPageMeta = (FPhysicalPageMetaData)0;
		if (bValidPageAddress)
		{
			pPageMeta = PhysicalPageMetaData[VSMPhysicalPageAddressToIndex(pPage)];
		}

		float4 Accumulator = OutVisualize[PixelPos];

		if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_FACTOR )
		{
			float4 Lit = float4(1,1,0,1);
			float4 Shadow = float4(0,0,0.5,1);
			Output = lerp(Shadow, Lit, Result.ShadowFactor);

			if (!bOneLightVisible)
			{
				// todo: figure out which is the least confusing blend mode
				Output = Accumulator + Output*0.25;
				Output.b = min(Output.b, 0.12); // a bit of blue helps retain contrast in shadowed areas
			}
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_OR_MIP )
		{
			// todo: consider separating out clipmap
			// todo: consider changing color mapping so near/far is easier to tell apart
			float3 Color = IntToColor( Result.ClipmapOrMipLevel );
			Output = float4(0.8f*Color + 0.2f*Result.ShadowFactor.xxx, 1.0);

			if (!bOneLightVisible)
			{
				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_VIRTUAL_PAGE && bValidPageAddress )
		{
			float3 MipColor = IntToColor( Result.ClipmapOrMipLevel );
			Output = float4(0.4f*vPageColor + 0.4f*MipColor + 0.2f*Result.ShadowFactor.xxx, 1.0);

			if (!bOneLightVisible)
			{
				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}		
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_SMRT_RAY_COUNT )
		{
			Output.xy = Accumulator.xy + float2(Result.RayCount, GetSMRTRayCount());
			Output.zw = float2(0, 1.0);
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_CACHED_PAGE && bValidPageAddress)
		{
			bool bUncachedDynamic = (pPageMeta.Flags & VSM_FLAG_DYNAMIC_UNCACHED) != 0;
			bool bUncachedStatic = (pPageMeta.Flags & VSM_FLAG_STATIC_UNCACHED) != 0;
			bool bForceCached = (pPageMeta.Flags & VSM_EXTENDED_FLAG_FORCE_CACHED) != 0;
			
			float CacheAlpha = bOneLightVisible ? 0.9 : 0.25;
			float VPageAlpha = 0;
			float ShadowAlpha = 0.1;

			float4 CacheColor = float4(0, 0, 0, 0);
			
			// Red = both uncached, blue = only static cached
			if (bUncachedDynamic)
			{
				if (!bVisualizeCachedPagesOnly)
				{
					CacheColor = bUncachedStatic ? float4(1, 0, 0, CacheAlpha) : float4(0, 0, 1, CacheAlpha);
				}
			}
			else if (bForceCached)
			{
				if (!bVisualizeCachedPagesOnly)
				{
					// TODO: Maybe only enable this debug color when "advanced" visualizations are enabled?
					// Could be confusing to content developers.
					CacheColor = float4(0.75, 1, 0, CacheAlpha);
				}
			}
			else if (bOneLightVisible || bVisualizeCachedPagesOnly)
			{
				CacheColor = float4(0, 1, 0, CacheAlpha); // Green if cached
				VPageAlpha = 0.1;
			}

			if (CacheColor.a > 0)
			{
				Output.rgb = CacheColor.a*CacheColor.rgb + VPageAlpha*vPageColor + ShadowAlpha*Result.ShadowFactor.xxx;
				Output = float4(Output.rgb + Accumulator.rgb, 0.9f);
			}
			else
			{
				Output = float4(0, 0, 0, 0);
			}
		}
		else if (VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_DIRTY_PAGE)
		{
			float3 PageColor = vPageColor * 0.4f;
			float3 DirtyColor = PageColor;

			if (!bOneLightVisible)
			{
				PageColor = Grayscale(PageColor);
			}

			if (bValidPageAddress)
			{
				// TODO: Rethink some of these modes...
				bool bStaticDirty = (pPageMeta.Flags & VSM_EXTENDED_FLAG_STATIC_DIRTY) != 0U;
				bool bDynamicDirty = (pPageMeta.Flags & VSM_EXTENDED_FLAG_DYNAMIC_DIRTY) != 0U;

				if (bStaticDirty)
				{
					DirtyColor.x = 1.0f;
				}
				if (bDynamicDirty)
				{
					DirtyColor.y = 1.0f;
				}
				PageColor = lerp(PageColor, DirtyColor, 0.8f);
			}
			Output = float4(PageColor, 1.0);

			if (!bOneLightVisible)
			{
				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}
		else if (VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_GPU_INVALIDATED_PAGE)
		{
			float3 CacheColor = float3(0, 0, 0);
			if (bValidPageAddress)
			{
				bool bInvalidatedDynamic = (pPageMeta.Flags & VSM_EXTENDED_FLAG_INVALIDATE_DYNAMIC) != 0;
				bool bInvalidatedStatic = (pPageMeta.Flags & VSM_EXTENDED_FLAG_INVALIDATE_STATIC) != 0;

				if (bInvalidatedStatic)
				{
					CacheColor.x = 1.0f;
				}
				if (bInvalidatedDynamic)
				{
					CacheColor.y = 1.0f;
				}
			}

			if (bOneLightVisible)
			{
				Output = float4(0.55f*CacheColor + 0.25f*vPageColor + 0.2f*Result.ShadowFactor.xxx, 1.0);
			}
			else
			{
				Output = float4(0.55f*CacheColor + 0.25f * Grayscale(vPageColor) + 0.2f*Result.ShadowFactor.xxx, 1.0);

				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_MERGED_PAGE)
		{
			// TODO: Maybe rethink this mode a bit
			bool bMerged = ((pPageMeta.Flags & VSM_EXTENDED_FLAG_STATIC_DIRTY) != 0U);
			if (bOneLightVisible)
			{
				float3 Color = float3(0, 1, 0);
				if (bMerged)
				{
					Color = float3(1, 0, 0);
				}
				Output = float4(0.55f * Color + 0.25f * vPageColor + 0.2f * Result.ShadowFactor.xxx, 1.0);
			}
			else
			{
				float3 Color = float3(0, 0, 0);
				if (bMerged)
				{
					Color = float3(1, 0, 0);
				}
				Output = float4(0.55f * Color + 0.25f * Grayscale(vPageColor) + 0.2f * Result.ShadowFactor.xxx, 1.0);

				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_GENERAL_DEBUG )
		{
			Output = float4(0.8f*Result.GeneralDebug + 0.2f*Result.ShadowFactor.xxx, 1.0);
			
			//Output.xyz = GreenToRedTurbo(Result.OccluderDistance / 1000.0f);
			Output.xyz = (Result.OccluderDistance / 2000.0f).xxx;
			Output.a = 1.0;

			if (!bOneLightVisible)
			{
				Output = Accumulator + Output*0.25;
				Output.a = 1.0;
			}
		}
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW )
		{
			// This is just to get the raw physical address
			FVirtualShadowMapSampleResult RawResult = SampleVirtualShadowMapTranslatedWorld(
				VirtualShadowMapId, TranslatedWorldPosition);

			const float OverdrawCount = float(VirtualShadowMap.PhysicalPagePool.Load(uint4(RawResult.PhysicalTexelAddress.xy, 2, 0))); // See FVisBufferPixel::WriteOverdraw
			Output = float4(OverdrawCount, 0, 0, 1.0);

			if (!bOneLightVisible)
			{
				Output.r += Accumulator.r;
			}
		}
		/* TODO: Enable this and add to menu
		else if ( VisualizeModeId == VIRTUAL_SHADOW_MAP_VISUALIZE_OCCLUDER_DISTANCE )
		{
			Output = (Result.OccluderDistance / 1000.0f).xxx;
		}
		*/
	}

	if (Output.a > 0)
	{
		OutVisualize[PixelPos] = Output;
	}
}

#if USE_TILE_LIST
uint bUseTileList;
Buffer<uint> TileListData;
#endif

uint2 GetTileCoord(uint2 GroupId)
{
	uint2 Out = GroupId.xy;
	#if USE_TILE_LIST
	IF_UBERSHADER(bUseTileList)
	{
		Out = UnpackTileCoord12bits(TileListData.Load(GroupId.x)); // Need to match SingleLayerWater tile encoding
	}
	#endif // USE_TILE_LIST
	return Out;
}

[numthreads(WORK_TILE_SIZE, WORK_TILE_SIZE, 1)]
void VirtualShadowMapProjection(
	uint3	GroupId				: SV_GroupID,
	uint	GroupIndex			: SV_GroupIndex,
	uint3	DispatchThreadId	: SV_DispatchThreadID )
{
	const uint2 TileCoord = GetTileCoord(GroupId.xy);

	// Morton order within a group so page access/atomics are more coherent and wave-swizzled gradients are possible.
	uint2 LocalPixelPos = WORK_TILE_SIZE * TileCoord + MortonDecode(GroupIndex);
	uint2 PixelPos = LocalPixelPos + uint2( ProjectionRect.xy );
	if ( any( PixelPos >= uint2( ProjectionRect.zw ) ) )
	{
		return;
	}
	
	float DeviceZ = SceneTexturesStruct.SceneDepthTexture.Load( int3( PixelPos, 0 ) ).r;
	const bool bIsHairInput = InputType == INPUT_TYPE_HAIRSTRANDS;
	#if HAS_HAIR_STRANDS
	if (bIsHairInput)
	{
		DeviceZ = HairStrands.HairOnlyDepthTexture.Load(int3(PixelPos, 0)).x;
		if (DeviceZ == 0)
		{
			return;
		}
	}
	#endif
	const float SceneDepth = ConvertFromDeviceZ( DeviceZ );

	const float4 SvPosition = float4( float2( PixelPos ) + 0.5, DeviceZ, 1.0f );
	const float3 TranslatedWorldPosition = SvPositionToTranslatedWorld( SvPosition );

	const float ScreenRayLengthWorld = GetScreenRayLengthMultiplierForProjectionType(VirtualShadowMap.ScreenRayLength * SceneDepth).y;
	const float Noise = BlueNoiseScalar(SvPosition.xy, View.StateFrameIndex);

	const FProjectionShadingInfo ShadingInfo = GetProjectionShadingInfo(PixelPos);

	FSubsurfaceOpacityMFP SubsurfaceOpacityMFP = GetInitialisedSubsurfaceOpacityMFP();
	if (!bIsHairInput && ShadingInfo.bIsSubsurface)
	{
		SubsurfaceOpacityMFP = GetSubsurfaceOpacityFromGbuffer(PixelPos);
	}

#if ONE_PASS_PROJECTION || UBERSHADER
	IF_UBERSHADER(bOnePassProjection)
	{
		const FCulledLightsGridHeader LightGridHeader = VirtualShadowMapGetLightsGridHeader(LocalPixelPos, SceneDepth);	
		// Allow one extra loop iteration to detect potential overflow
		const uint LoopCount = min(GetPackedShadowMaskMaxLightCount() + 1, LightGridHeader.NumLights);

		uint4 ShadowMask = InitializePackedShadowMask();
		LOOP
		for (uint Index = 0; Index < LoopCount; ++Index)
		{
			const FLocalLightData LightData = VirtualShadowMapGetLocalLightData(LightGridHeader, Index);
		
			// Megalights are sorted to the end (See LightSceneInfo.h SortyKey) so if we hit one we're done
			if (UnpackIsHandledByMegaLights(LightData))
			{
				break;
			}

			if (Index < GetPackedShadowMaskMaxLightCount())
			{
				// VSM pruned light grid, so this should always be valid
				int VirtualShadowMapId = LightData.Internal.VirtualShadowMapId;

				FLightShaderParameters Light = ConvertFromLocal(LightData);
				FVirtualShadowMapSampleResult Result = ProjectLight(
					VirtualShadowMapId,
					Light,
					ShadingInfo,
					PixelPos,
					SceneDepth,
					ScreenRayLengthWorld,
					TranslatedWorldPosition,
					Noise,
					SubsurfaceOpacityMFP);

				FilterVirtualShadowMapSampleResult(PixelPos, Result);
				PackShadowMask(ShadowMask, Result.ShadowFactor, Index);

				#if VISUALIZE_OUTPUT
				if (bVisualizeOutput)
				{
					OutputVisualize( VirtualShadowMapId, PixelPos - View.ViewRectMin.xy, TranslatedWorldPosition, Result );
				}
				#endif // VISUALIZE_OUTPUT
			}
			else
			{
				// Record overflow
				StatsBufferInterlockedEnableFlags(VSM_STAT_OVERFLOW_FLAGS, VSM_STAT_OVERFLOW_FLAG_OPP_MAX_LIGHTS, true, true);
			}
		}

		OutShadowMaskBits[PixelPos] = ShadowMask;
	}
#endif // ONE_PASS_PROJECTION || UBERSHADER

#if !ONE_PASS_PROJECTION || UBERSHADER
	IF_UBERSHADER(!bOnePassProjection)
	{
		int VirtualShadowMapId = LightUniformVirtualShadowMapId;
		FLightShaderParameters Light = GetRootLightShaderParameters();

#if DIRECTIONAL_LIGHT
		// Increases light source radius for subsurface material as the opacity reduces to emulates light diffusion.
		// TODO: Does something similar make sense for local lights?
		// Substrate: this approximation has no concrete foundation so it is currently skipped for Substrate using PerPixel SSS, e.g. when bDataIsOpacity==false and data is MFP.
		if (SubsurfaceOpacityMFP.bDataIsOpacity && SubsurfaceOpacityMFP.Data < 1.0f)
		{
			const float SubsurfaceOpacity = SubsurfaceOpacityMFP.Data;
			Light.SourceRadius = max(Light.SourceRadius, (1.0f - SubsurfaceOpacity) * SubsurfaceMinSourceRadius);
		}
#endif

		FVirtualShadowMapSampleResult Result = ProjectLight(
			VirtualShadowMapId,
			Light,
			ShadingInfo,
			PixelPos,
			SceneDepth,
			ScreenRayLengthWorld,
			TranslatedWorldPosition,
			Noise,
			SubsurfaceOpacityMFP);

		OutShadowFactor[ PixelPos ] = Result.ShadowFactor.xx;

		#if VISUALIZE_OUTPUT
		IF_UBERSHADER(bVisualizeOutput)
		{
			OutputVisualize(VirtualShadowMapId, PixelPos - View.ViewRectMin.xy, TranslatedWorldPosition, Result);
		}
		#endif // VISUALIZE_OUTPUT

	}
#endif // !ONE_PASS_PROJECTION || UBERSHADER
}
