// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapProjectionCommon.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapProjectionStructs.ush"
#include "/Engine/Shared/LightDefinitions.h"

#define VIRTUAL_SHADOW_MAP_MIN_OCCLUDER_DISTANCE 1e-6f

#ifndef VSM_WITH_DOF_BIAS
	#define VSM_WITH_DOF_BIAS 0
#endif

#if VSM_WITH_DOF_BIAS
#include "../DiaphragmDOF/DOFCommon.ush"

float DOFBiasStrength;

// Areas that are out of focus get a lower VSM resolution, as the resulting image will get blurred anyway
float CalculateResolutionBiasFromDepthOfField(float SceneDepth)
{
	bool bApplyBiasFromDOFCoCRadius = DOFBiasStrength != 0.0f;
	if (bApplyBiasFromDOFCoCRadius)
	{
		float AbsCocRadius = abs(SceneDepthToCocRadius(SceneDepth));
		const float BiasRampUpDistance = 8.0f;
		return min(DOFBiasStrength, lerp(0.0f, DOFBiasStrength, AbsCocRadius / BiasRampUpDistance));
	}
	else
	{
		return 0.0f;
	}
}
#endif

float CalcAbsoluteClipmapLevel(float DistanceToClipmapOriginSquared)
{
	// log(X*X) <=> 2 * log(X) - though certain compilers had this one covered already
	return log2(DistanceToClipmapOriginSquared) * 0.5f;
}

// Optionally apply the resolution bias (rather than greedy sampling highest res data) depending on the global setting
float CalcBiasedAbsoluteClipmapLevelForSampling(FVirtualShadowMapProjectionShaderData BaseProjectionData, float DistanceToClipmapOriginSquared, float SceneDepth)
{
	float AbsoluteLevel = CalcAbsoluteClipmapLevel(DistanceToClipmapOriginSquared);

	float BiasedLevel = AbsoluteLevel; //todo: what about other bias sources, like VirtualShadowMap.GlobalResolutionLodBias

	if (SceneDepth >= 0)
	{
#if VSM_WITH_DOF_BIAS
		float DOFResolutionBias = CalculateResolutionBiasFromDepthOfField(SceneDepth);
		BiasedLevel += DOFResolutionBias;
#endif
	}

	int ClipmapIndex = max(0, BiasedLevel - BaseProjectionData.ClipmapLevel);
	if (ClipmapIndex < BaseProjectionData.ClipmapLevelCountRemaining)
	{
		const FVirtualShadowMapHandle VSMHandle = BaseProjectionData.VirtualShadowMapHandle.MakeOffset(ClipmapIndex);
		float PerVSMBias = GetVirtualShadowMapProjectionData(VSMHandle).ResolutionLodBias;
		BiasedLevel += PerVSMBias;
	}
	else
	{
		BiasedLevel += BaseProjectionData.ResolutionLodBias;
	}

	return BiasedLevel;
}

float CalcDistanceToClipmapOriginSquared(FVirtualShadowMapProjectionShaderData BaseProjectionData, float3 TranslatedWorldPosition)
{
	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(BaseProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 TranslatedWorldOrigin = -BaseProjectionData.ClipmapWorldOriginOffset + ViewToShadowTranslation;
	return length2(TranslatedWorldPosition + TranslatedWorldOrigin);
}

float CalcAbsoluteClipmapLevel(FVirtualShadowMapProjectionShaderData BaseProjectionData, float3 TranslatedWorldPosition)
{
	return CalcAbsoluteClipmapLevel(CalcDistanceToClipmapOriginSquared(BaseProjectionData, TranslatedWorldPosition));
}

float CalcBiasedAbsoluteClipmapLevelForSampling(FVirtualShadowMapProjectionShaderData BaseProjectionData, float3 TranslatedWorldPosition, float SceneDepth)
{
	return CalcBiasedAbsoluteClipmapLevelForSampling(BaseProjectionData, CalcDistanceToClipmapOriginSquared(BaseProjectionData, TranslatedWorldPosition), SceneDepth);
}

// Calculate the projected pixel footprint onto a local light, scaled in shadow map texels
// ShadowTranslatedWorldPosition should already be translated by both the primary view and shadow ProjectionData.PreViewTranslation
float VirtualShadowMapCalcPixelFootprintLocal(
	FVirtualShadowMapProjectionShaderData ProjectionData,
	float3 ShadowTranslatedWorldPosition,
	float SceneDepth)
{
	// Compute footprint by projecting the approximate size of a camera pixel at the given depth to shadow space
	// NOTE: This doesn't take the screen XY position/FOV into account, which may or may not be desirable.

	// TODO: Roll into a uniform
	float2 RadiusXY = 1.0f / (View.ViewSizeAndInvSize.xy * View.ViewToClip._m00_m11);
	float RadiusScreen = min(RadiusXY.x, RadiusXY.y);
	float DepthScale = SceneDepth * View.ViewToClip[2][3] + View.ViewToClip[3][3];

	float RadiusWorld = DepthScale * RadiusScreen;

	float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);

	float4 RadiusClipH = mul(float4(RadiusWorld, 0.0f, ShadowUVz.w, 1.0f), ProjectionData.ShadowViewToClipMatrix);
	float RadiusClip = abs(RadiusClipH.x / RadiusClipH.w);
	float Footprint = RadiusClip * float(2 * VSM_VIRTUAL_MAX_RESOLUTION_XY);

	return Footprint;
}

uint VirtualShadowMapGetMipLevelLocalFromReceiver(
	FVirtualShadowMapProjectionShaderData ProjectionData,
	float3 ReceiverShadowTranslatedWorld,
	float ReceiverSceneDepth)
{
	uint MipLevel = 0;
	if (ProjectionData.bUseReceiverMask)
	{
		float Footprint = VirtualShadowMapCalcPixelFootprintLocal(ProjectionData, ReceiverShadowTranslatedWorld, ReceiverSceneDepth);
		return GetMipLevelLocal(Footprint, VirtualShadowMap.MipModeLocal, ProjectionData.ResolutionLodBias, VirtualShadowMap.GlobalResolutionLodBias);
	}
	else
	{
		return 0;
	}
}

struct FVirtualShadowMapSample
{
	float Depth;
	uint MipLevel;
	FVirtualShadowMapHandle VirtualShadowMapHandle;		// May be offset from input for clipmaps
	bool bValid;
	uint2 VirtualTexelAddress;
	float2 VirtualTexelAddressFloat;
	uint2 PhysicalTexelAddress;
};

FVirtualShadowMapSample InitVirtualShadowMapSample()
{
	FVirtualShadowMapSample Result;
	Result.Depth = 0.0f;
	Result.MipLevel = 0;
	Result.VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeInvalid(); 
	Result.bValid = false;
	Result.VirtualTexelAddress = Result.PhysicalTexelAddress = uint2(0U, 0U);
	Result.VirtualTexelAddressFloat = float2(0.0f, 0.0f);
	return Result;
}

float SampleVirtualShadowMapPhysicalDepth(uint2 PhysicalTexelAddress)
{
	return asfloat(VirtualShadowMap.PhysicalPagePool.Load(uint4(PhysicalTexelAddress, 0, 0)));
}

FVirtualShadowMapSample SampleVirtualShadowMapLevel(FVirtualShadowMapHandle VirtualShadowMapHandle, float2 ShadowMapUV, uint MipLevel)
{
	FVirtualShadowMapSample Result = InitVirtualShadowMapSample();

	Result.VirtualTexelAddressFloat = ShadowMapUV * float(CalcLevelDimsTexels(MipLevel));
	Result.VirtualTexelAddress = uint2(Result.VirtualTexelAddressFloat);

	bool PagePresent = VirtualToPhysicalTexel(VirtualShadowMapHandle, MipLevel, Result.VirtualTexelAddress, Result.PhysicalTexelAddress);
	if (PagePresent)
	{
		Result.Depth = SampleVirtualShadowMapPhysicalDepth(Result.PhysicalTexelAddress);
		Result.MipLevel = MipLevel;
		Result.VirtualShadowMapHandle = VirtualShadowMapHandle;
		Result.bValid = true;
	}
	
	return Result;
}

FVirtualShadowMapSample SampleVirtualShadowMap(FVirtualShadowMapHandle VirtualShadowMapHandle, float2 ShadowMapUV, uint MinMipLevel)
{
	FShadowPageTranslationResult Page = ShadowVirtualToPhysicalUV(VirtualShadowMapHandle, ShadowMapUV, MinMipLevel);
	FVirtualShadowMapSample Result = InitVirtualShadowMapSample();
	if (Page.bValid)
	{
		Result.bValid = true;
		Result.MipLevel = Page.LODOffset;
		Result.VirtualShadowMapHandle = VirtualShadowMapHandle;
		Result.VirtualTexelAddress = Page.VirtualTexelAddress;
		Result.VirtualTexelAddressFloat = Page.VirtualTexelAddressFloat;
		Result.PhysicalTexelAddress = Page.PhysicalTexelAddress;
		Result.Depth = SampleVirtualShadowMapPhysicalDepth(Result.PhysicalTexelAddress);
	}

	return Result;
}

// VSM normal bias that smoothly scales with distance to camera
float3 VirtualShadowMapGetNormalBiasLength(float3 TranslatedWorldPosition)
{
	float DistanceToCamera = GetDistanceToCameraFromViewVector(TranslatedWorldPosition - View.TranslatedWorldCameraOrigin);
	float NormalBiasLength = max(0.02f, VirtualShadowMap.NormalBias * DistanceToCamera / GetCotanHalfFieldOfView().x);
	return NormalBiasLength;
}

// Data to convert UV's and depths from one clipmap level basis to another
struct FVirtualShadowMapClipmapRelativeTransform
{
	float Scale;
	float3 Bias;
};

// Resulting transform takes UV's/depths from clipmap level ClipmapId to ClipmapId + LevelOffset 
// ClipmapId + LevelOffset should be a valid level in the same clipmap
// See VirtualShadowMapClipmap.cpp for construction details
FVirtualShadowMapClipmapRelativeTransform CalcClipmapRelativeTransform(FVirtualShadowMapHandle ClipmapHandle, int LevelOffset)
{
	const FVirtualShadowMapProjectionShaderData ProjectionDataA = GetVirtualShadowMapProjectionData(ClipmapHandle);
	const FVirtualShadowMapProjectionShaderData ProjectionDataB = GetVirtualShadowMapProjectionData(ClipmapHandle.MakeOffset(LevelOffset));

	float2 OffsetA = float2(ProjectionDataA.ClipmapCornerRelativeOffset);
	float2 OffsetB = float2(ProjectionDataB.ClipmapCornerRelativeOffset);

	FVirtualShadowMapClipmapRelativeTransform Result;
	Result.Scale = LevelOffset >= 0 ? rcp(float(1U << LevelOffset)) : float(1U << (-LevelOffset));
	Result.Bias.xy = 0.25f * (OffsetB - Result.Scale * OffsetA);

	// NOTE: relative Z bias can change when caching is enabled due to cached levels pinning the depth range
	float OffsetZA = ProjectionDataA.ShadowViewToClipMatrix[3][2];
	float OffsetZB = ProjectionDataB.ShadowViewToClipMatrix[3][2];
	Result.Bias.z = OffsetZB - Result.Scale * OffsetZA;

	return Result;
}

// Transforms a virtual page in a given clipmap level to a page in the offset coarser level
// This offset must be positive or else the page would not be guaranteed to exist in the target level
// This is done entirely in integer math as well to avoid precision issues when looking up coarser fallback pages
uint2 CalcClipmapOffsetLevelPage(uint2 BasePage, FVirtualShadowMapHandle ClipmapHandle, uint LevelOffset)
{
	const FVirtualShadowMapProjectionShaderData ProjectionDataA = GetVirtualShadowMapProjectionData(ClipmapHandle);
	const FVirtualShadowMapProjectionShaderData ProjectionDataB = GetVirtualShadowMapProjectionData(ClipmapHandle.MakeOffset(LevelOffset));

	const int OffsetScale = (VSM_LEVEL0_DIM_PAGES_XY >> 2);
	int2 BasePageOffset  = OffsetScale * ProjectionDataA.ClipmapCornerRelativeOffset;
	int2 LevelPageOffset = OffsetScale * ProjectionDataB.ClipmapCornerRelativeOffset;
	return (BasePage - BasePageOffset + (LevelPageOffset << LevelOffset)) >> LevelOffset;
}

// Will sample from the given clipmap level Id, onwards to coarser levels if no valid data is present
// ShadowMapUVs are in terms of the given clipmap level/virtual shadow map
FVirtualShadowMapSample SampleVirtualShadowMapClipmap(FVirtualShadowMapHandle VirtualShadowMapHandle, float2 ShadowMapUV)
{
	FVirtualShadowMapSample Result = InitVirtualShadowMapSample();

#define DEBUG_VSM_CLIPMAP_LEVEL_SEARCH 0
#if !DEBUG_VSM_CLIPMAP_LEVEL_SEARCH
	uint2 BasePage = uint2(ShadowMapUV * VSM_LEVEL0_DIM_PAGES_XY);
	FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(VirtualShadowMapHandle, 0, BasePage));
	if (PhysicalPageEntry.bAnyLODValid)
	{	
		uint ClipmapLevelOffset = PhysicalPageEntry.LODOffset;
		FVirtualShadowMapHandle ClipmapLevelHandle = VirtualShadowMapHandle.MakeOffset(ClipmapLevelOffset);

		Result.VirtualTexelAddressFloat = ShadowMapUV * float(CalcLevelDimsTexels(0));
		Result.VirtualTexelAddress = uint2(Result.VirtualTexelAddressFloat);
		float DepthLevelScale = 1.0f;
		float DepthLevelBias = 0.0f;

		// Need to use a coarser clipmap to find a valid page
		if (ClipmapLevelOffset > 0)
		{
			// Compute the virtual page in the offset level by integer math and clamp to the edges to avoid any precision
			// issues at borders that may cause us to miss the mapped page.
			uint2 vPage = CalcClipmapOffsetLevelPage(BasePage, VirtualShadowMapHandle, ClipmapLevelOffset);
			uint2 VirtualTexelAddressMin = vPage * VSM_PAGE_SIZE;
			uint2 VirtualTexelAddressMax = VirtualTexelAddressMin + (VSM_PAGE_SIZE - 1);

			FVirtualShadowMapClipmapRelativeTransform Transform = CalcClipmapRelativeTransform(VirtualShadowMapHandle, ClipmapLevelOffset);
			float2 ClipmapUV = ShadowMapUV * Transform.Scale + Transform.Bias.xy;
			DepthLevelScale = Transform.Scale;
			DepthLevelBias = Transform.Bias.z;

			// NOTE: Do not clamp the float address as it messes with optimal slope bias calculations
			Result.VirtualTexelAddressFloat = ClipmapUV * float(CalcLevelDimsTexels(0));
			Result.VirtualTexelAddress = clamp(uint2(Result.VirtualTexelAddressFloat), VirtualTexelAddressMin, VirtualTexelAddressMax);

			PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(ClipmapLevelHandle, 0, vPage));
		}

		// NOTE: This really should be valid if we got here
		if (PhysicalPageEntry.bThisLODValid)
		{
			Result.PhysicalTexelAddress = 
				PhysicalPageEntry.PhysicalAddress * VSM_PAGE_SIZE +
				(Result.VirtualTexelAddress & VSM_PAGE_SIZE_MASK);

			// Convert depth into back into the original reference clipmap level range
			Result.Depth = (SampleVirtualShadowMapPhysicalDepth(Result.PhysicalTexelAddress) - DepthLevelBias) / DepthLevelScale;
			Result.MipLevel = 0;
			Result.VirtualShadowMapHandle = ClipmapLevelHandle;
			Result.bValid = true;
		}
	}
#else // DEBUG_VSM_CLIPMAP_LEVEL_SEARCH
	FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	const int MaxOffset = BaseProjectionData.ClipmapLevelCountRemaining;
	for (int Offset = 0; Offset < MaxOffset; ++Offset)
	{
		FVirtualShadowMapClipmapRelativeTransform Transform = CalcClipmapRelativeTransform(VirtualShadowMapHandle, Offset);		
		float2 ClipmapUV = ShadowMapUV * Transform.Scale + Transform.Bias.xy;
		Result = SampleVirtualShadowMapLevel(VirtualShadowMapHandle.MakeOffset(Offset), ClipmapUV, 0);
		if (Result.bValid)
		{
			// Convert depth into back into the original reference clipmap level range
			Result.Depth = (Result.Depth - Transform.Bias.z) / Transform.Scale;
			break;
		}
	}
#endif // DEBUG_VSM_CLIPMAP_LEVEL_SEARCH

	return Result;
}

float ComputeVirtualShadowMapOptimalSlopeBias(
	// Used to compare to the sampled SmSample.VirtualShadowMapId to adjust depth scale for clipmaps
	FVirtualShadowMapHandle RequestedVirtualShadowMapHandle,
	FVirtualShadowMapSample SmSample,
	float3 TranslatedWorldPosition,
	float3 EstimatedGeoWorldNormal,
	bool bClamp = true)
{
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(SmSample.VirtualShadowMapHandle);

	// Transform geometry world-space plane eq to shadow 'UV' texture space [0-1] ranges
	float4 NormalPlaneTranslatedWorld = float4(EstimatedGeoWorldNormal, -dot(EstimatedGeoWorldNormal, TranslatedWorldPosition));
	float4 NormalPlaneUV = mul(NormalPlaneTranslatedWorld, ProjectionData.TranslatedWorldToShadowUVNormalMatrix);

	float2 DepthSlopeUV = -NormalPlaneUV.xy / NormalPlaneUV.z;
	float MipLevelDim = float(CalcLevelDimsTexels(SmSample.MipLevel));
	float2 TexelCenter = float2(SmSample.VirtualTexelAddress) + 0.5f;
	float2 TexelCenterOffset = TexelCenter - SmSample.VirtualTexelAddressFloat;
	float2 TexelCenterOffsetUV = TexelCenterOffset / MipLevelDim;
	// 2x factor due to lack of precision (probably)
	float OptimalSlopeBias = 2.0f * max(0.0f, dot(DepthSlopeUV, TexelCenterOffsetUV));
	
	// Clamp to avoid excessive degenerate slope biases causing flickering lit pixels
	OptimalSlopeBias = bClamp ? min(OptimalSlopeBias, abs(100.0f * ProjectionData.ShadowViewToClipMatrix._33)) : OptimalSlopeBias;

	// Adjust depth scale if we sampled a different clipmap level
	// NOTE: Sampled clipmap should always be >= the requested one (coarser)
	// Do this after clamping to be consistent in world space
	OptimalSlopeBias *= float(1u << (SmSample.VirtualShadowMapHandle.Id - RequestedVirtualShadowMapHandle.Id));

	return OptimalSlopeBias;
}

// Used for orthographic projections (i.e. directional lights)
// Receiver depth is post-projection-divide.
float ComputeOccluderDistanceOrtho(float4x4 ShadowViewToClip, float OccluderDepth, float ReceiverDepth)
{
	float OccluderViewZ = (OccluderDepth - ShadowViewToClip._43) / ShadowViewToClip._33;
	float ReceiverViewZ = (ReceiverDepth - ShadowViewToClip._43) / ShadowViewToClip._33;

	// No perspective projection, so simple difference gets us the distance
	float Result = ReceiverViewZ - OccluderViewZ;
	return max(VIRTUAL_SHADOW_MAP_MIN_OCCLUDER_DISTANCE, Result);
}

// Used for perspective projections (i.e. spot lights)
// Receiver depth is post-projection-divide.
float ComputeOccluderDistancePerspective(float4x4 ShadowViewToClip, float OccluderDepth, float ReceiverDepth, float ReceiverDistance)
{
	float OccluderViewZ = ShadowViewToClip._43 / (OccluderDepth - ShadowViewToClip._33);
	float ReceiverViewZ = ShadowViewToClip._43 / (ReceiverDepth - ShadowViewToClip._33);

	// Similar triangles to compute euclidean distance in view/world space
	float OccluderDistance = (ReceiverDistance / ReceiverViewZ) * OccluderViewZ;
	float Result = ReceiverDistance - OccluderDistance;	
	return max(VIRTUAL_SHADOW_MAP_MIN_OCCLUDER_DISTANCE, Result);
}

uint VirtualShadowMapGetCubeFace( float3 Dir )
{
	// TODO use v_cubeid_f32( Dir )
	if( abs(Dir.x) >= abs(Dir.y) && abs(Dir.x) >= abs(Dir.z) )
		return Dir.x > 0 ? 0 : 1;
	else if( abs(Dir.y) > abs(Dir.z) )
		return Dir.y > 0 ? 2 : 3;
	else
		return Dir.z > 0 ? 4 : 5;
}

struct FVirtualShadowMapSampleResult
{
	bool bValid;
	float ShadowFactor;	// 0 = fully occluded, 1 = no occlusion
	float OccluderDistance;

	// Debug data; do not reference in non-debug permutations or performance may be affected!
	int ClipmapOrMipLevel;		// Absolute clipmap level or mip level
	uint RayCount;
	uint2 VirtualTexelAddress;
	uint2 PhysicalTexelAddress;
	float3 GeneralDebug;		// General purpose debug output during shader development
};


FVirtualShadowMapSampleResult InitVirtualShadowMapSampleResult()
{
	FVirtualShadowMapSampleResult Result;
	Result.bValid = false;
	Result.ShadowFactor = 1.0f;
	Result.OccluderDistance = -1.0f;
	Result.ClipmapOrMipLevel = 0;
	Result.VirtualTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	Result.PhysicalTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	Result.RayCount = 0;
	Result.GeneralDebug = float3(0, 0, 0);
	return Result;
}


/**
* Sample virtual shadow (clip)map for a directional light
*
* VirtualShadowMapId is the ID of the virtual shadow map to sample
* TranslatedWorldPosition is the sample position in PrimaryViews translated world space to project into the shadow map
* RayStartDistance is an optional offset to move the lookup along the shadow ray towards the light
*   - Should be zero or positive
*   - This offset is useful in that it does *not* affect the selection of clipmap level, unlike offsetting the WorldPosition itself
*   - OccluderDistance will still be relative to the original sample position
* EstimatedGeoWorldNormal is ideally the geometric (flat) normal of the sample point in world space
*   - If bUseOptimalBias is true, this is used to compute a receiver-plane-based bias for the sample point
*   - The shading normal can be used if the geometric normal is not available, but divergence from the geometric normal can cause biasing issues.
* SceneDepth is the distance from the sample point to the main view, or -1 if not available.
*/
FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectional(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 TranslatedWorldPosition,
	float RayStartDistance,
	float3 EstimatedGeoWorldNormal,
	bool bUseOptimalBias = true,
	float SceneDepth = -1.0f)
{
	RayStartDistance = max(RayStartDistance, 0.0f);

	FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	checkSlow(BaseProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL);
	
	const int ClipmapLevel = int(floor(CalcBiasedAbsoluteClipmapLevelForSampling(BaseProjectionData, TranslatedWorldPosition, SceneDepth)));
	int ClipmapIndex = max(0, ClipmapLevel - BaseProjectionData.ClipmapLevel);

	// Check if sample is within the clipmap range (from camera)
	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	if (ClipmapIndex < BaseProjectionData.ClipmapLevelCountRemaining)
	{
		FVirtualShadowMapHandle ClipmapLevelVirtualShadowMapHandle = VirtualShadowMapHandle.MakeOffset(ClipmapIndex);
		FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(ClipmapLevelVirtualShadowMapHandle);

		// No perspective divided needed for ortho projection
		float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
		float3 ShadowTranslatedWorldPosition = TranslatedWorldPosition + ViewToShadowTranslation;
		float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);

		FVirtualShadowMapSample SmSample;
		SmSample = SampleVirtualShadowMapClipmap(ClipmapLevelVirtualShadowMapHandle, ShadowUVz.xy);

		if (SmSample.bValid)
		{
			int SampledClipmapIndex = SmSample.VirtualShadowMapHandle.Id - VirtualShadowMapHandle.Id;

			Result.bValid = true;
			Result.ShadowFactor = 1.0f;
			Result.OccluderDistance = -1.0f;
			Result.ClipmapOrMipLevel = GetVirtualShadowMapProjectionData(SmSample.VirtualShadowMapHandle).ClipmapLevel;
			Result.VirtualTexelAddress = SmSample.VirtualTexelAddress;
			Result.PhysicalTexelAddress = SmSample.PhysicalTexelAddress;
			Result.RayCount = 1;

			float OptimalSlopeBias = 0.0f;
			BRANCH
			if (bUseOptimalBias)
			{
				OptimalSlopeBias = ComputeVirtualShadowMapOptimalSlopeBias(ClipmapLevelVirtualShadowMapHandle, SmSample, ShadowTranslatedWorldPosition, EstimatedGeoWorldNormal);
			}

			float RayStartBias = -RayStartDistance * ProjectionData.ShadowViewToClipMatrix._33;
			float BiasedDepth = SmSample.Depth - OptimalSlopeBias - RayStartBias;

			if (BiasedDepth > ShadowUVz.z)
			{
				Result.ShadowFactor = 0.0f;
				Result.OccluderDistance = ComputeOccluderDistanceOrtho(
					ProjectionData.ShadowViewToClipMatrix,
					SmSample.Depth,
					ShadowUVz.z);
			}
		}
	}

	return Result;
}

FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectional(
	int VirtualShadowMapId,
	float3 TranslatedWorldPosition,
	float RayStartDistance,
	float3 EstimatedGeoWorldNormal,
	bool bUseOptimalBias = true,
	float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapDirectional(
		FVirtualShadowMapHandle::MakeFromIdDirectional(VirtualShadowMapId),
		TranslatedWorldPosition,
		RayStartDistance,
		EstimatedGeoWorldNormal,
		bUseOptimalBias,
		SceneDepth);
}

/**
* Sample virtual shadow map for a local light
* 
* Parameter descriptions are as above, except:
* - SceneDepth can be optionally provided to allow computing the appropriate mip level for sampling when
*	using receiver mask to avoid sampling incomplete data from more detailed mip levels.
*/
FVirtualShadowMapSampleResult SampleVirtualShadowMapLocal(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 TranslatedWorldPosition,
	float RayStartDistance,
	float3 EstimatedGeoWorldNormal,
	bool bUseOptimalBias = true,
	float SceneDepth = -1.0f)
{
	RayStartDistance = max(RayStartDistance, 0.0f);

	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 ShadowTranslatedWorldPosition = TranslatedWorldPosition + ViewToShadowTranslation;

	if (ProjectionData.LightType != LIGHT_TYPE_SPOT)
	{
		VirtualShadowMapHandle = VirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(ShadowTranslatedWorldPosition));
		ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	}

	// TODO cubemap math directly instead of vector fetching matrix
	float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
	ShadowUVz.xyz /= ShadowUVz.w;

	// When using receiver mask, we also must compute the base mip level to use
	// We cannot use greedy mip selection in this case since the higher res mips may not have the rendered
	// geometry that covers the requested mask at this receiver. We must use a similar-or-lower mip than
	// the one that we marked.
	uint MipLevel = 0;
	if (SceneDepth >= 0.0f)
	{
		MipLevel = VirtualShadowMapGetMipLevelLocalFromReceiver(ProjectionData, ShadowTranslatedWorldPosition, SceneDepth);
	}

	FVirtualShadowMapSample SmSample = SampleVirtualShadowMap(VirtualShadowMapHandle, ShadowUVz.xy, max(MipLevel, ProjectionData.MinMipLevel));
	if (SmSample.bValid)
	{
		FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
		Result.bValid = true;
		Result.ShadowFactor = 1.0f;
		Result.OccluderDistance = -1.0f;
		Result.ClipmapOrMipLevel = int(SmSample.MipLevel);
		Result.VirtualTexelAddress = SmSample.VirtualTexelAddress;
		Result.PhysicalTexelAddress = SmSample.PhysicalTexelAddress;
		Result.RayCount = 1;

		float OptimalSlopeBias = 0.0f;
		BRANCH
		if (bUseOptimalBias)
		{
			OptimalSlopeBias = ComputeVirtualShadowMapOptimalSlopeBias(VirtualShadowMapHandle, SmSample, ShadowTranslatedWorldPosition, EstimatedGeoWorldNormal);
		}

		float RayStartBias = -RayStartDistance * ProjectionData.ShadowViewToClipMatrix._33 / ShadowUVz.w;
		float BiasedDepth = SmSample.Depth - OptimalSlopeBias - RayStartBias;

		if (BiasedDepth > ShadowUVz.z)
		{
			Result.ShadowFactor = 0.0f;
			// Shadow view matrix is rotation
			float ReceiverDistance = length(ShadowTranslatedWorldPosition);
			Result.OccluderDistance = ComputeOccluderDistancePerspective(
				ProjectionData.ShadowViewToClipMatrix,
				BiasedDepth,
				ShadowUVz.z,
				ReceiverDistance);
		}

		return Result;
	}

	// Invalid
	return InitVirtualShadowMapSampleResult();
}

FVirtualShadowMapSampleResult SampleVirtualShadowMapLocal(
	int VirtualShadowMapId,
	float3 TranslatedWorldPosition,
	float RayStartDistance,
	float3 EstimatedGeoWorldNormal,
	bool bUseOptimalBias = true,
	float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapLocal(
		FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId),
		TranslatedWorldPosition,
		RayStartDistance,
		EstimatedGeoWorldNormal,
		bUseOptimalBias,
		SceneDepth);
}

// Some common variants/convenience functions

// Sample in PrimaryView translated world space with no optimal bias
FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectional(int VirtualShadowMapId, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f, float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapDirectional(VirtualShadowMapId, TranslatedWorldPosition, RayStartDistance, float3(0, 0, 0), false, SceneDepth);
}
FVirtualShadowMapSampleResult SampleVirtualShadowMapDirectional(FVirtualShadowMapHandle VirtualShadowMapHandle, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f, float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapDirectional(VirtualShadowMapHandle, TranslatedWorldPosition, RayStartDistance, float3(0, 0, 0), false, SceneDepth);
}
FVirtualShadowMapSampleResult SampleVirtualShadowMapLocal(int VirtualShadowMapId, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f, float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapLocal(VirtualShadowMapId, TranslatedWorldPosition, RayStartDistance, float3(0, 0, 0), false, SceneDepth);
}
FVirtualShadowMapSampleResult SampleVirtualShadowMapLocal(FVirtualShadowMapHandle VirtualShadowMapHandle, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f, float SceneDepth = -1.0f)
{
	return SampleVirtualShadowMapLocal(VirtualShadowMapHandle, TranslatedWorldPosition, RayStartDistance, float3(0, 0, 0), false, SceneDepth);
}



/**
* Sample a virtual shadow map for an unknown light type with no optimal bias.
* Will route to the appropriate function above.
* Generally the directional/local variants should be used when the light's type is known to cut down on the
* code/register overhead of including both paths.
*/
FVirtualShadowMapSampleResult SampleVirtualShadowMapTranslatedWorld(FVirtualShadowMapHandle VirtualShadowMapHandle, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f)
{
	FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);
	if (BaseProjectionData.LightType == LIGHT_TYPE_DIRECTIONAL)
	{
		return SampleVirtualShadowMapDirectional(VirtualShadowMapHandle, TranslatedWorldPosition, RayStartDistance);
	}
	else
	{
		return SampleVirtualShadowMapLocal(VirtualShadowMapHandle, TranslatedWorldPosition, RayStartDistance);
	}
}

/**
* Sample a virtual shadow map for an unknown light type with no optimal bias.
* Will route to the appropriate function above.
* Generally the directional/local variants should be used when the light's type is known to cut down on the
* code/register overhead of including both paths.
*/
FVirtualShadowMapSampleResult SampleVirtualShadowMapTranslatedWorld(int VirtualShadowMapId, float3 TranslatedWorldPosition, float RayStartDistance = 0.0f)
{
	return SampleVirtualShadowMapTranslatedWorld(FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId), TranslatedWorldPosition, RayStartDistance);
}

struct FVirtualShadowMapFilterableContext
{
	Texture2D<uint> PageTable;
	Texture2D<uint> PageReceiverMasks;
	Texture2D<float> PrefilteredDistantPhysicalPagePool;
	bool bHavePrevReceiverMasks;
	ByteAddressBuffer ProjectionDataBuffer;

	FShadowPhysicalPage VirtualToPhysicalTexelBase(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
	{
		uint VPageX = VirtualTexelAddress.x >> VSM_LOG2_PAGE_SIZE;
		uint VPageY = VirtualTexelAddress.y >> VSM_LOG2_PAGE_SIZE;

		FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(PageTableLevelOffset, Level, uint2(VPageX, VPageY)), PageTable);
		PhysicalTexelAddress = PhysicalPageEntry.PhysicalAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE + (VirtualTexelAddress & VSM_PAGE_SIZE_MASK) + VSM_PREFILTERED_DISTANT_BORDER_SIZE;
		return PhysicalPageEntry;
	}

	// Returns true if the page is valid at all
	bool VirtualToPhysicalTexel(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, uint2 VirtualTexelAddress, inout uint2 PhysicalTexelAddress)
	{
		return VirtualToPhysicalTexelBase(PageTableLevelOffset, Level, VirtualTexelAddress, PhysicalTexelAddress).bIsSecondaryRequest;
	}

	// Returns true if the page is valid *for rendering*
	FShadowPhysicalPage VirtualToPhysicalTexelBase(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, float2 VirtualTexelAddressFloat, inout float2 PhysicalTexelAddressFloat)
	{
		uint2 VirtualTexelAddress = uint2(floor(VirtualTexelAddressFloat));
		uint2 vPage = VirtualTexelAddress >> VSM_LOG2_PAGE_SIZE;
		float2 TexelPosInPage = VirtualTexelAddressFloat - float2(vPage << VSM_LOG2_PAGE_SIZE);

		FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(CalcPageOffset(PageTableLevelOffset, Level, vPage), PageTable);
		PhysicalTexelAddressFloat = float2(PhysicalPageEntry.PhysicalAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE + VSM_PREFILTERED_DISTANT_BORDER_SIZE) + TexelPosInPage;// + TexelOffset;
		return PhysicalPageEntry;
	}

	// Returns true if the page is valid at all
	bool VirtualToPhysicalTexel(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, float2 VirtualTexelAddressFloat, inout float2 PhysicalTexelAddressFloat)
	{
		return VirtualToPhysicalTexelBase(PageTableLevelOffset, Level, VirtualTexelAddressFloat, PhysicalTexelAddressFloat).bIsSecondaryRequest;
	}

	// Returns true if the page is valid and the receiver mask is set for the given texel (or is disabled)
	bool VirtualToPhysicalTexelReceiverMask(FVirtualSMLevelOffset PageTableLevelOffset, uint Level, float2 VirtualTexelAddressFloat, inout float2 PhysicalTexelAddressFloat)
	{
		uint2 VirtualTexelAddress = uint2(floor(VirtualTexelAddressFloat));
		uint2 vPage = VirtualTexelAddress >> VSM_LOG2_PAGE_SIZE;
		float2 TexelPosInPage = VirtualTexelAddressFloat - float2(vPage << VSM_LOG2_PAGE_SIZE);

		FVSMPageOffset PageOffset = CalcPageOffset(PageTableLevelOffset, Level, vPage);

		BRANCH
		if (bHavePrevReceiverMasks)
		{
			// 4x4 sub mask
			uint2 MaskAddress = (VirtualTexelAddress >> (VSM_LOG2_PAGE_SIZE - VSM_LOG2_RECEIVER_MASK_SIZE)) & VSM_RECEIVER_MASK_SUBMASK;
			// 2x2 quadrant mask
			uint2 MaskQuadrant = (VirtualTexelAddress >> (VSM_LOG2_PAGE_SIZE - 1u) & 1u);

			if ((PageReceiverMasks[PageOffset.GetResourceAddress() * 2u + MaskQuadrant] & (1u << (MaskAddress.y * 4u + MaskAddress.x))) == 0u)
			{
				return false;
			}
		}

		FShadowPhysicalPage PhysicalPageEntry = ShadowGetPhysicalPage(PageOffset, PageTable);
		PhysicalTexelAddressFloat = float2(PhysicalPageEntry.PhysicalAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE + VSM_PREFILTERED_DISTANT_BORDER_SIZE) + TexelPosInPage;
		return PhysicalPageEntry.bIsSecondaryRequest;
	}

	bool SampleBilinearReceiverMask(FVirtualSMLevelOffset PageTableLevelOffset, float2 VirtualTexelAddressFloat, inout float OutShadowFactor)
	{
		float2 PhysicalTexelAddressFloat;
		if (VirtualToPhysicalTexelReceiverMask(PageTableLevelOffset, 0u, VirtualTexelAddressFloat, PhysicalTexelAddressFloat))
		{
			float2 PhysicalTexelUV = PhysicalTexelAddressFloat * VirtualShadowMap.RecFilterablePhysicalPoolSize.xy;

			OutShadowFactor = PrefilteredDistantPhysicalPagePool.SampleLevel(VirtualShadowMap.BilinearClampedSampler, PhysicalTexelUV, 0);	
			return true;
		}
		return false;
	}

	bool SampleBilinear(FVirtualSMLevelOffset PageTableLevelOffset, float2 VirtualTexelAddressFloat, inout float OutShadowFactor)
	{
		float2 PhysicalTexelAddressFloat;
		if (VirtualToPhysicalTexel(PageTableLevelOffset, 0u, VirtualTexelAddressFloat, PhysicalTexelAddressFloat))
		{
			float2 PhysicalTexelUV = PhysicalTexelAddressFloat * VirtualShadowMap.RecFilterablePhysicalPoolSize.xy;

			OutShadowFactor = PrefilteredDistantPhysicalPagePool.SampleLevel(VirtualShadowMap.BilinearClampedSampler, PhysicalTexelUV, 0);	
			return true;
		}
		return false;
	}

	bool SampleBilinear(FShadowPhysicalPage PhysicalPageEntry, float2 VirtualTexelAddressFloat, inout float OutShadowFactor)
	{
		float2 TexelPosInPage = fmod(VirtualTexelAddressFloat, float(VSM_PAGE_SIZE));

		float2 PhysicalTexelAddressFloat = float2(PhysicalPageEntry.PhysicalAddress * VSM_PREFILTERED_DISTANT_PAGE_SIZE + VSM_PREFILTERED_DISTANT_BORDER_SIZE) + TexelPosInPage;// + TexelOffset;
		float2 PhysicalTexelUV = PhysicalTexelAddressFloat * VirtualShadowMap.RecFilterablePhysicalPoolSize.xy;

		OutShadowFactor = PrefilteredDistantPhysicalPagePool.SampleLevel(VirtualShadowMap.BilinearClampedSampler, PhysicalTexelUV, 0);	
		return true;
	}
};

float2 ViewTranslatedWorldToVirtualTexel(FVirtualShadowMapProjectionShaderData ProjectionData, float3 TranslatedWorldPositionView, uint MipLevel = 0u)
{
	float3 ShadowTranslatedWorldPosition = TranslatedWorldPositionView + ProjectionData.ClipmapWorldOriginOffset;
	float4 ShadowUVZW = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
	return ShadowUVZW.xy * float(CalcLevelDimsTexels(MipLevel));
}
