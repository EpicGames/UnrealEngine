// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "../LightShaderParameters.ush"
#include "VirtualShadowMapMaskBitsCommon.ush"

#ifndef WORK_TILE_SIZE
#define WORK_TILE_SIZE 8
#endif

Buffer<uint> TileListData;

void VirtualShadowMapCompositeTileVS(
	in uint InstanceId : SV_InstanceID,
	in uint VertexId : SV_VertexID,
	out float4 Position : SV_POSITION)
{
	const uint TileData = TileListData[InstanceId.x];
	const uint2 TileOrigin = WORK_TILE_SIZE * UnpackTileCoord12bits(TileData);

	uint2 TileVertex = TileOrigin;
	TileVertex.x += VertexId == 1 || VertexId == 2 || VertexId == 4 ? WORK_TILE_SIZE : 0;
	TileVertex.y += VertexId == 2 || VertexId == 4 || VertexId == 5 ? WORK_TILE_SIZE : 0;

	// Viewport is set on the view rect so no offset is needed.
	Position = float4(float2(TileVertex) * View.ViewSizeAndInvSize.zw * float2(2.0f, -2.0f) + float2(-1.0, 1.0f), 0.5f, 1.0f);
}

Texture2D<float2> InputShadowFactor;
Texture2D<float> InputDepthTexture;
Texture2D<uint4> ShadowMaskBits;
int CompositeVirtualShadowMapId;
uint bModulateRGB;

void VirtualShadowMapCompositePS(
	in float4 SvPosition : SV_Position,
	out float4 OutShadowMask : SV_Target)
{
	float2 ShadowFactor = InputShadowFactor.Load(int3(SvPosition.xy, 0));
	OutShadowMask = (bModulateRGB != 0) ? ShadowFactor.xxxx : EncodeLightAttenuation(float4(ShadowFactor.x, ShadowFactor.y, ShadowFactor.x, ShadowFactor.y));
}

void VirtualShadowMapCompositeFromMaskBitsPS(
	in float4 SvPosition : SV_Position,
	out float4 OutShadowMask : SV_Target)
{
	const uint2 PixelPos = uint2(SvPosition.xy);

	const float DeviceZ = InputDepthTexture.Load(int3(PixelPos, 0)).r;
	const float SceneDepth = ConvertFromDeviceZ(DeviceZ);

	// Possibly needed to do fallback single sample VSM lookup
	const float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);

	float ShadowMask = GetVirtualShadowMapMaskForLight(
		ShadowMaskBits[PixelPos], PixelPos, SceneDepth, CompositeVirtualShadowMapId, TranslatedWorldPosition);
			
	OutShadowMask = EncodeLightAttenuation(ShadowMask.xxxx);
}
