// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapProjectionDirectional.ush:
=============================================================================*/
#pragma once

#include "../DeferredShadingCommon.ush"
#include "../SceneTexturesCommon.ush"
#include "../LightShaderParameters.ush"
#include "../Visualization.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapSMRTCommon.ush"

float2 ComputeDepthSlopeDirectionalUV(
	FVirtualShadowMapProjectionShaderData ProjectionData,
	float3 EstimatedGeoWorldNormal,
	bool bClamp = true)
{
	float4 NormalPlaneUV = mul(float4(EstimatedGeoWorldNormal, 0.0f), ProjectionData.TranslatedWorldToShadowUVNormalMatrix);
	float2 DepthSlopeUV = -NormalPlaneUV.xy / NormalPlaneUV.z;

	// Clamp to avoid excessive degenerate slope biases causing flickering lit pixels
	float2 Clamp = 0.05f;
	DepthSlopeUV = bClamp ? clamp(DepthSlopeUV, -Clamp, Clamp) : DepthSlopeUV;

	return DepthSlopeUV;
}

float ComputeOptimalSlopeBiasDirectional(
	float2 DepthSlopeUV,
	float2 OffsetUV)
{
	return 2.0f * max(0.0f, dot(DepthSlopeUV, OffsetUV));
}

FVirtualShadowMapHandle GetMappedClipmap(FVirtualShadowMapHandle VirtualShadowMapHandle, float3 RayOriginTranslatedWorld, float SceneDepth)
{
	// This is the (unbiased) most detailed clipmap level that we are guaranteed to have a page table entry for
	const FVirtualShadowMapProjectionShaderData BaseProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);		
	float ClipmapLevelFloat = CalcBiasedAbsoluteClipmapLevelForSampling(BaseProjectionData, RayOriginTranslatedWorld, SceneDepth);
	int ClipmapIndex = max(0, int(floor(ClipmapLevelFloat)) - BaseProjectionData.ClipmapLevel);

	if (ClipmapIndex >= BaseProjectionData.ClipmapLevelCountRemaining)
	{
		return FVirtualShadowMapHandle::MakeInvalid();
	}

	// Clipmap origin should be near shaded samples, so shadow translated world should be regular range
	FVirtualShadowMapHandle ClipmapHandle = VirtualShadowMapHandle.MakeOffset(ClipmapIndex);
	const FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(ClipmapHandle);
	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 RayOriginShadowTranslatedWorld = RayOriginTranslatedWorld + ViewToShadowTranslation;
	float3 RayStartUVZ = mul(float4(RayOriginShadowTranslatedWorld, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix).xyz;

	// NOTE: We don't need the actual sample here, we're just looking for the best mapped page at the ray origin
	// Trusting a lot in DCE here... may be better to refactor at some point
	FVirtualShadowMapSample ShadowSample = SampleVirtualShadowMapClipmap(ClipmapHandle, RayStartUVZ.xy);
	if (ShadowSample.bValid && ShadowSample.VirtualShadowMapHandle.Id > ClipmapHandle.Id)
	{
		// NOTE: We could bias this (as long as we clamp it to 0 still) to allow sampling higher resolution
		// data in the case where we just happened to hit a pixel with lower resolution, but note that performance
		// suffers when we have to use fallback levels in the SMRT loop, so this is fine for now.
		// We may have to revisit this if we start getting finer grained with caching and mapped page resolutions,
		// but likely if this matters here we would already be seeing visible resolution seams between pages regardless.
		ClipmapIndex += (ShadowSample.VirtualShadowMapHandle.Id - ClipmapHandle.Id);
	}

	return VirtualShadowMapHandle.MakeOffset(ClipmapIndex);
}

struct FSMRTClipmapRayState
{
	FVirtualShadowMapHandle ClipmapHandle;			// Pre-offset virtual shadow map ID of specific clipmap level
	float3 RayStartUVZ;	
	float3 RayStepUVZ;
	float ExtrapolateSlope;
	uint2 PhysicalTexelAddress;
	uint2 VirtualTexelAddress;
	FVirtualShadowMapHandle SampledClipmapHandle;
};

FSMRTClipmapRayState SMRTClipmapRayInitialize(
	const FVirtualShadowMapProjectionShaderData ProjectionData,
	float3 RayOriginShadowTranslatedWorld,
	float3 RayDir,
	float RayLength,
	float RayStartOffset,
	float2 DepthSlopeUV,
	float2 TexelOffset,
	float ExtrapolateSlope)
{
	float3 RayStart = RayOriginShadowTranslatedWorld + RayDir * RayStartOffset;
	float3 RayVector = RayDir * RayLength;

	float3 RayStartUVZ = mul(float4(RayStart, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix).xyz;
	float3 RayStepUVZ  = mul(float4(RayVector, 0.0f), ProjectionData.TranslatedWorldToShadowUVMatrix).xyz;
	
	// Texel dither to hide aliasing
	// Note that this is directly scaled in texel space of the clipmap level, so it can create visual
	// discontinuities at clipmap boundaries. We could scale this by distance instead of texel size but
	// in the case where texels are large enough to be obvious there are already visual discontinuities.
	{
		float OptimalBias = ComputeOptimalSlopeBiasDirectional(DepthSlopeUV, TexelOffset);
		// Subtract off any portion of the bias that was already covered by the ray start offset (usually screen ray)
		OptimalBias = max(0.0f, OptimalBias - abs(RayStartOffset * ProjectionData.ShadowViewToClipMatrix._33));

		RayStartUVZ.xy += TexelOffset;
		RayStartUVZ.z += OptimalBias;
	}

	FSMRTClipmapRayState Result = (FSMRTClipmapRayState)0;
	Result.ClipmapHandle = ProjectionData.VirtualShadowMapHandle;
	Result.RayStartUVZ = RayStartUVZ;
	Result.RayStepUVZ = RayStepUVZ;
	// Max depth slope for slope-based extrapolation when using SMRT. Scale so that it doesn't change based on ZRangeScale.
	Result.ExtrapolateSlope = abs(ExtrapolateSlope * ProjectionData.ShadowViewToClipMatrix._33);
	Result.VirtualTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	Result.PhysicalTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	return Result;
}

FSMRTSample SMRTFindSample(inout FSMRTClipmapRayState RayState, float SampleTime)
{	
	const float3 SampleUVZ = RayState.RayStartUVZ + RayState.RayStepUVZ * SampleTime;
	FVirtualShadowMapSample ShadowSample = SampleVirtualShadowMapClipmap(RayState.ClipmapHandle, SampleUVZ.xy);

	FSMRTSample Sample = InitSMRTSample();
	Sample.bValid = ShadowSample.bValid;
	Sample.ReferenceDepth = SampleUVZ.z;
	Sample.ExtrapolateSlope = RayState.ExtrapolateSlope;

	if (ShadowSample.bValid)
	{
		Sample.SampleDepth = ShadowSample.Depth;

		// Debug
		RayState.VirtualTexelAddress = ShadowSample.VirtualTexelAddress;
		RayState.PhysicalTexelAddress = ShadowSample.PhysicalTexelAddress;
		RayState.SampledClipmapHandle = ShadowSample.VirtualShadowMapHandle;
	}

	return Sample;
}

// Instantiate SMRTRayCast for FSMRTClipmapRayState
#define SMRT_TEMPLATE_RAY_STRUCT FSMRTClipmapRayState
#include "VirtualShadowMapSMRTTemplate.ush"
#undef SMRT_TEMPLATE_RAY_STRUCT


float3 GetRandomDirectionalLightRayDir(FLightShaderParameters Light, float2 E)
{
	float3 RayDir = Light.Direction;
	{
		float2 DiskUV = UniformSampleDiskConcentric(E) * Light.SourceRadius;
		float3 N = RayDir;
		float3 dPdu = cross(N, (abs(N.x) > 1e-6f) ? float3(1, 0, 0) : float3(0, 1, 0));
		float3 dPdv = cross(dPdu, N);
		RayDir += dPdu * DiskUV.x + dPdv * DiskUV.y;
	}
	return normalize(RayDir);
}

// Normal and LightDirection should be normalized
bool IsBackfaceToDirectionalLight(float3 Normal, float3 LightDirection, float LightSourceRadius)
{
	// Allow a minimum of ~5 degrees of wiggle room to account for normal issues
	float MinSinAlpha = 0.1;
	float SinAlpha = max(abs(LightSourceRadius), MinSinAlpha);
	return dot(Normal, LightDirection) < -SinAlpha;
}

FVirtualShadowMapSampleResult TraceDirectional(
	int VirtualShadowMapId,
	FLightShaderParameters Light,
	uint2 PixelPos,
	const float SceneDepth,
	float3 TranslatedWorldPosition,
	float RayStartOffset,
	const float Noise,
	float3 WorldNormal,
	const FSMRTTraceSettings Settings = GetSMRTTraceSettingsDirectional())
{
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromIdDirectional(VirtualShadowMapId);
	float3 ViewPosition = mul(float4(TranslatedWorldPosition, 1.0f), View.TranslatedWorldToView).xyz;
	float DistanceFromViewOrigin = length(ViewPosition);

	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	Result.bValid = true;	// TODO: false if all samples of all rays miss pages?
	Result.ShadowFactor = 1.0f;

	// Find the best resolution mapped clipmap at the ray origin location
	const FVirtualShadowMapHandle ClipmapHandle = GetMappedClipmap(VirtualShadowMapHandle, TranslatedWorldPosition, SceneDepth);
	if (!ClipmapHandle.IsValid())
	{
		// TODO: False for valid probably? But make sure that doesn't do anything weird as we have no fallback here
		Result.ShadowFactor = 1.0f;		// Fully lit outside clipmap range
		return Result;
	}

	const FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(ClipmapHandle);

	// This function is designed to vary in depth in roughly the same way as shadow texel density does,
	// thus the dependence on Clipmap ResolutionLodBias. The goal here is to define a texel dither that
	// is "smooth" and doesn't have obvious artifacts at page edges of different clipmap levels.
	// 
	// This could become a problem later if we break too far from that global resolution bias. That said,
	// these cases would have unavoidable discontinuities at page edges, so that would have to be handled
	// in some different way.
	//
	// 0.5 is arbitrary but makes it consistent with the previous dither scale cvar
	float PerLightTexelDitherScale = Settings.TexelDitherScale * ProjectionData.TexelDitherScale;
	float DitherScale = 0.0f;
	if (PerLightTexelDitherScale > 0.0f)
	{
		DitherScale =
			((0.5f / float(CalcLevelDimsTexels(0))) * PerLightTexelDitherScale * DistanceFromViewOrigin) /
			(exp2(ProjectionData.ClipmapLevel - ProjectionData.ResolutionLodBias));
	}

	float2 DepthSlopeUV = ComputeDepthSlopeDirectionalUV(ProjectionData, WorldNormal);

	// Compute max ray length based on view depth
	// This affects both how far we are willing to trace across the shadow map (for clipmaps this is related to view depth)
	// and the maximum size a penumbra can be.
	// Too high values will cause shadows to detach from their contact points (unless more samples are used).
	// Too low values will greatly restrict how large penumbras can be in screen space.
	float RayLength = VirtualShadowMap.SMRTRayLengthScale * DistanceFromViewOrigin;

	// Clipmap origin should be near shaded samples, so shadow translated world should be regular range
	float3 ViewToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	float3 RayOriginShadowTranslatedWorld = TranslatedWorldPosition + ViewToShadowTranslation;

	uint RayMissCount = 0;
	uint i = 0;
	float OccluderDistanceSum = 0.0f;
	float MaxOccluderDistance = -1.0f;
	const uint MaxRayCount = Settings.RayCount;
	for ( ; i < MaxRayCount; i++)
	{
		// One sample for ray, one for texel dither
		float4 RandSample = VirtualShadowMapGetRandomSample(PixelPos, View.StateFrameIndex, i, MaxRayCount);
		float3 RayDir = GetRandomDirectionalLightRayDir(Light, RandSample.xy);
		float2 TexelOffset = (RandSample.zw - 0.5f) * DitherScale;

		FSMRTClipmapRayState RayState = SMRTClipmapRayInitialize(
			ProjectionData,
			RayOriginShadowTranslatedWorld,
			RayDir,
			RayLength,
			RayStartOffset,
			DepthSlopeUV,
			TexelOffset,
			Settings.ExtrapolateMaxSlope);
		FSMRTResult SMRTResult = SMRTRayCast(RayState, Settings.SamplesPerRay, Noise);

		// Debug output (DCE'd if not used)
		Result.ClipmapOrMipLevel = GetVirtualShadowMapProjectionData(RayState.SampledClipmapHandle).ClipmapLevel;
		Result.VirtualTexelAddress = RayState.VirtualTexelAddress;
		Result.PhysicalTexelAddress =  RayState.PhysicalTexelAddress;
		//Result.GeneralDebug = GreenToRedTurbo((RayState.SampledClipmapId - RayState.ClipmapId) / 2.0f);
			
		if (SMRTResult.bValidHit)
		{
			float OccluderDistance = ComputeOccluderDistanceOrtho(
				GetVirtualShadowMapProjectionData(RayState.ClipmapHandle).ShadowViewToClipMatrix,
				SMRTResult.HitDepth,
				RayState.RayStartUVZ.z);

			OccluderDistanceSum += OccluderDistance;
			MaxOccluderDistance = max(MaxOccluderDistance, OccluderDistance);
		}
		else
		{
			++RayMissCount;
		}

		#if COMPUTESHADER
		if (Settings.AdaptiveRayCount > 0)
		{
			// TODO: Adapt this heuristic based on SMRTAdaptiveRayCount as well?
			if( i == 0 )
			{
				bool bHit = SMRTResult.bValidHit;

				// All lanes missed
				bool bAllLanesMiss = WaveActiveAllTrue( !bHit );
				if( bAllLanesMiss )
				{
					break;
				}
			}
			else if (i >= Settings.AdaptiveRayCount)
			{
				// After 2 iterations and all have hit, assume umbra
				bool bAllLanesHit = WaveActiveAllTrue( RayMissCount == 0 );
				if( bAllLanesHit )
				{
					break;
				}
			}
		}
		#endif
	}
	uint RayCount = min(i + 1U, MaxRayCount);		// break vs regular for loop exit

	/* TODO: Experiment with max vs avg distance in various cases
	float OccluderDistance = (View.GeneralPurposeTweak == 0)
		? MaxOccluderDistance
		: (OccluderDistanceSum / float(max(1, RayCount - RayMissCount)));
	*/
	float OccluderDistance = (OccluderDistanceSum / float(max(1U, RayCount - RayMissCount)));

	Result.ShadowFactor = float(RayMissCount) / float(RayCount);
	Result.OccluderDistance = OccluderDistance;
	Result.RayCount = RayCount;
	return Result;
}

// Generate a ray based on directional light source geometry (e.g, source radius)
bool GenerateRayDirectional(
	FLightShaderParameters Light,
	uint2 PixelPos,
	float3 TranslatedWorldPosition,
	float RayLengthScale,
	uint RayIndex,
	uint RayCount,
	inout float3 OutRayStart,
	inout float3 OutRayEnd)
{
	float3 ViewPosition = mul(float4(TranslatedWorldPosition, 1.0f), View.TranslatedWorldToView).xyz;
	float DistanceFromViewOrigin = length(ViewPosition);

	// Compute max ray length based on view depth
	// This affects both how far we are willing to trace across the shadow map (for clipmaps this is related to view depth)
	// and the maximum size a penumbra can be.
	// Too high values will cause shadows to detach from their contact points (unless more samples are used).
	// Too low values will greatly restrict how large penumbras can be in screen space.
	float RayLength = RayLengthScale * DistanceFromViewOrigin;

	float2 E = VirtualShadowMapGetRandomSample(PixelPos, View.StateFrameIndex, RayIndex, RayCount).xy;

	float3 RayDir = GetRandomDirectionalLightRayDir(Light, E);
	OutRayStart = TranslatedWorldPosition;
	OutRayEnd = OutRayStart + RayDir * RayLength;
	return true;
}
