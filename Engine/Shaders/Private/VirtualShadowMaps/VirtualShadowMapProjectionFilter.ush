// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapProjectionFilter.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../BlueNoise.ush"
#include "VirtualShadowMapProjectionCommon.ush"

void FilterVirtualShadowMapSampleResult(uint2 PixelPos, inout FVirtualShadowMapSampleResult In)
{
	// Add some simple dither to break up banding
	// This is not perfect for stuff like subsurface stuff in motion, as the banding on those smooth
	// gradients can be significant and in motion TSR can't get enough samples to average it all out,
	// but it's inexpensive amd usually pretty reasonable.
	// Current noise scale aligns with one pass projection packing, but also works well for typical SMRT ray counts
	// The tolerance on the lower end is somewhat arbitrary, but needed to trade off brightening umbra regions in subsurface
	// materials (due to adding noise then clamping) against banding in penumbra regions on these surfaces.
	const float NoiseScale = (1.0f / 15.0f);
	if (In.ShadowFactor > (NoiseScale/4.0f) && In.ShadowFactor < 1.0f)
	{
		In.ShadowFactor += (BlueNoiseScalar(PixelPos, View.StateFrameIndex) - 0.5f) * NoiseScale;
		In.ShadowFactor = saturate(In.ShadowFactor);
	}
}
