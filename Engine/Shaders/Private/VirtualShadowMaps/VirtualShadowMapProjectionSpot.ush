// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapProjectionSpot.ush:
=============================================================================*/
#pragma once

#include "../DeferredShadingCommon.ush"
#include "../SceneTexturesCommon.ush"
#include "../LightShaderParameters.ush"
#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "VirtualShadowMapPageAccessCommon.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "VirtualShadowMapSMRTCommon.ush"

float2 ComputeDepthSlopeLocalUV(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 ShadowTranslatedWorldPosition,
	float3 WorldNormal)
{
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

	float4 NormalPlaneTranslatedWorld = float4(WorldNormal, -dot(WorldNormal, ShadowTranslatedWorldPosition));
	float4 NormalPlaneUV = mul(NormalPlaneTranslatedWorld, ProjectionData.TranslatedWorldToShadowUVNormalMatrix);
	float2 DepthSlopeUV = -NormalPlaneUV.xy / NormalPlaneUV.z;
	return DepthSlopeUV;
}

float ComputeOptimalSlopeBiasLocal(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 ShadowTranslatedWorldPosition,
	float2 DepthSlopeUV,
	bool bClamp = true)
{
	// NOTE: Better to reload the necessary data here based on VSM ID than keep it alive in registers!
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

	float4 ShadowUVz = mul(float4(ShadowTranslatedWorldPosition, 1.0f), ProjectionData.TranslatedWorldToShadowUVMatrix);
	ShadowUVz.xyz /= ShadowUVz.w;
	FShadowPageTranslationResult Page = ShadowVirtualToPhysicalUV(ProjectionData.VirtualShadowMapHandle, ShadowUVz.xy, ProjectionData.MinMipLevel);

	float MipLevelDim = float(CalcLevelDimsTexels(Page.LODOffset));
	float2 TexelCenter = float2(Page.VirtualTexelAddress) + 0.5f;
	float2 TexelCenterOffset = TexelCenter - Page.VirtualTexelAddressFloat;
	float2 TexelCenterOffsetUV = TexelCenterOffset / MipLevelDim;
	float OptimalSlopeBias = 2.0f * max(0.0f, dot(DepthSlopeUV, TexelCenterOffsetUV));

	return OptimalSlopeBias;
}


struct FSMRTSingleRayState
{
	FVirtualShadowMapHandle VirtualShadowMapHandle;
	float3 RayStartUVz;	
	float3 RayStepUVz;
	float ExtrapolateSlope;
	uint MinMipLevel;
	// Debug output
	int MipLevel;
	uint2 VirtualTexelAddress;
	uint2 PhysicalTexelAddress;
};

FSMRTSingleRayState SMRTSingleRayInitialize(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	// Shadow PreViewTranslation already added in
	float3 RayStartShadowTranslatedWorld,
	float3 RayEndShadowTranslatedWorld,
	float ExtrapolateSlope,
	float DepthBias,
	uint MipLevel,
	bool bClampUVs = true)
{
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(VirtualShadowMapHandle);

	float4 RayStartUVz = mul(float4(RayStartShadowTranslatedWorld, 1), ProjectionData.TranslatedWorldToShadowUVMatrix).xyzw;
	float4 RayEndUVz   = mul(float4(RayEndShadowTranslatedWorld  , 1), ProjectionData.TranslatedWorldToShadowUVMatrix).xyzw;

	// NOTE: We assume by construction that ray ends are not behind the light near plane as the warping
	// due to SMRT ray tests gets severe long before the rays would otherwise clip at that plane.
	// If clipping is ever necessary, it must be done to the ray endpoints in world space so that lights that
	// need to walk multiple faces/clipmap levels in lock step can do it consistently.

	RayStartUVz.xyz = RayStartUVz.xyz / RayStartUVz.w;
	RayEndUVz.xyz = RayEndUVz.xyz / RayEndUVz.w;
	float3 RayStepUVz = RayEndUVz.xyz - RayStartUVz.xyz;

	// Offsets can move it slightly off the edge of spotlight projection. Clamp to edge.
	// Note we do *not* want to do this if we are tracing dual face cubemap rays!
	if (bClampUVs)
	{
		RayStartUVz.xy = saturate(RayStartUVz.xy);
	}
	RayStartUVz.z += DepthBias;

	FSMRTSingleRayState Result;
	Result.VirtualShadowMapHandle = VirtualShadowMapHandle;
	Result.RayStartUVz = RayStartUVz.xyz;
	Result.RayStepUVz = RayStepUVz.xyz;
	Result.ExtrapolateSlope = ExtrapolateSlope * RayStepUVz.z;
	Result.MinMipLevel = max(MipLevel, ProjectionData.MinMipLevel);
	Result.MipLevel = 0;
	Result.VirtualTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	Result.PhysicalTexelAddress = uint2(0xFFFFFFFF, 0xFFFFFFFF);
	return Result;
}

FSMRTSample SMRTFindSample(inout FSMRTSingleRayState RayState, float SampleTime)
{
	float3 UVz = RayState.RayStartUVz.xyz + RayState.RayStepUVz.xyz * SampleTime;

	FSMRTSample Sample = InitSMRTSample();
	Sample.bValid = false;
	Sample.ReferenceDepth = UVz.z;
	Sample.ExtrapolateSlope = RayState.ExtrapolateSlope;

	if (all(UVz.xy == saturate(UVz.xy)))
	{
		FVirtualShadowMapSample SmSample = SampleVirtualShadowMap(RayState.VirtualShadowMapHandle, UVz.xy, RayState.MinMipLevel);
		if (SmSample.bValid)
		{
			Sample.bValid = true;
			Sample.SampleDepth = SmSample.Depth;
			
			// Debug output
			RayState.MipLevel = SmSample.MipLevel;
			RayState.PhysicalTexelAddress = SmSample.PhysicalTexelAddress;
			RayState.VirtualTexelAddress = SmSample.VirtualTexelAddress;
		}
	}

	return Sample;
}

// Instantiate SMRTRayCast for FSMRTSingleRayState
#define SMRT_TEMPLATE_RAY_STRUCT FSMRTSingleRayState
#include "VirtualShadowMapSMRTTemplate.ush"
#undef SMRT_TEMPLATE_RAY_STRUCT


struct FSMRTTwoCubeFaceRayState
{
	FSMRTSingleRayState Face0;
	FSMRTSingleRayState Face1;
	bool bSampleInFace1;
};

FSMRTTwoCubeFaceRayState SMRTTwoCubeFaceRayInitialize(
	FVirtualShadowMapHandle VirtualShadowMapHandle0,
	FVirtualShadowMapHandle VirtualShadowMapHandle1,
	float3 RayStartShadowTranslatedWorld,
	float3 RayEndShadowTranslatedWorld,
	float ExtrapolateSlope,
	float DepthBias,
	uint MipLevel)
{
	FSMRTTwoCubeFaceRayState Result;
	Result.Face0 = SMRTSingleRayInitialize(VirtualShadowMapHandle0, RayStartShadowTranslatedWorld, RayEndShadowTranslatedWorld, ExtrapolateSlope, DepthBias, MipLevel, false);
	Result.Face1 = SMRTSingleRayInitialize(VirtualShadowMapHandle1, RayStartShadowTranslatedWorld, RayEndShadowTranslatedWorld, ExtrapolateSlope, DepthBias, MipLevel, false);
	Result.bSampleInFace1 = true;
	return Result;
}

FSMRTSample SMRTFindSample(inout FSMRTTwoCubeFaceRayState RayState, float SampleTime)
{
	// NOTE: Traces from END to START, so we start in face1
	FSMRTSample Sample = InitSMRTSample();
	if (RayState.bSampleInFace1)
	{
		Sample = SMRTFindSample(RayState.Face1, SampleTime);
		if (!Sample.bValid)
		{
			Sample = SMRTFindSample(RayState.Face0, SampleTime);
			Sample.bResetExtrapolation = true;
			RayState.bSampleInFace1 = false;
		}
	}
	else
	{
		Sample = SMRTFindSample(RayState.Face0, SampleTime);
	}
	return Sample;
}

// Instantiate SMRTRayCast for FSMRTTwoCubeFaceRayState
#define SMRT_TEMPLATE_RAY_STRUCT FSMRTTwoCubeFaceRayState
#include "VirtualShadowMapSMRTTemplate.ush"
#undef SMRT_TEMPLATE_RAY_STRUCT


FSMRTResult ShadowRayCastSpotLight(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 RayStartShadowTranslatedWorld,
	float3 RayEndShadowTranslatedWorld,
	float DepthBias,
	uint MipLevel,
	int NumSteps,
	float StepOffset,
	float ExtrapolateSlope,
	// To get debug data out
	inout FSMRTSingleRayState OutRayState)
{
	OutRayState = SMRTSingleRayInitialize(VirtualShadowMapHandle, RayStartShadowTranslatedWorld, RayEndShadowTranslatedWorld, ExtrapolateSlope, DepthBias, MipLevel);
	return SMRTRayCast(OutRayState, NumSteps, StepOffset);
}

FSMRTResult ShadowRayCastPointLight(
	FVirtualShadowMapHandle VirtualShadowMapHandle,
	float3 RayStartShadowTranslatedWorld,
	float3 RayEndShadowTranslatedWorld,
	float DepthBias,
	uint MipLevel,
	int NumSteps,
	float StepOffset,
	float ExtrapolateSlope,
	// To get debug data out
	inout FSMRTSingleRayState OutRayState)
{
	FVirtualShadowMapHandle RayStartFaceHandle	= VirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(RayStartShadowTranslatedWorld));
	FVirtualShadowMapHandle RayEndFaceHandle	= VirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(RayEndShadowTranslatedWorld));

	FSMRTResult Result;
	if (WaveActiveAnyTrue(RayStartFaceHandle.Id != RayEndFaceHandle.Id))
	{
		FSMRTTwoCubeFaceRayState RayState = SMRTTwoCubeFaceRayInitialize(RayStartFaceHandle, RayEndFaceHandle,
			RayStartShadowTranslatedWorld, RayEndShadowTranslatedWorld, ExtrapolateSlope, DepthBias, MipLevel);
		Result = SMRTRayCast(RayState, NumSteps, StepOffset);

		if (RayState.bSampleInFace1)
		{
			OutRayState = RayState.Face1;
		}
		else
		{
			OutRayState = RayState.Face0;
		}
	}
	else
	{
		// Fast path: ray stays on a single cube map face (effectively a spot light)
		OutRayState = SMRTSingleRayInitialize(RayEndFaceHandle, RayStartShadowTranslatedWorld, RayEndShadowTranslatedWorld, ExtrapolateSlope, DepthBias, MipLevel);
		Result = SMRTRayCast(OutRayState, NumSteps, StepOffset);
	}

	return Result;
}

// Normal should be normalized
bool IsBackfaceToLocalLight(
	float3 ToLight,
	float3 Normal,
	float LightSourceRadius)
{
	float DistSqr = dot(ToLight, ToLight);
	float Falloff = rcp(DistSqr + 1);
	float InvDist = rsqrt(DistSqr);
	float SinAlphaSqr = saturate(Pow2(LightSourceRadius) * Falloff);
	float SinAlpha = sqrt(SinAlphaSqr);
	bool bBackface = dot(Normal, ToLight * InvDist) < -SinAlpha;
	return bBackface;
}

// Shadow resolution scale that roughly matches the LOD function in VSM page marking
float ComputeShadowResolutionScale(FVirtualShadowMapProjectionShaderData ProjectionData, float SceneDepth)
{
	const float2 RadiusXY = 1.0f / (View.ViewSizeAndInvSize.xy * View.ViewToClip._11_22);
	const float RadiusScreen = min(RadiusXY.x, RadiusXY.y);
	const float DepthScale = SceneDepth * View.ViewToClip._34 + View.ViewToClip._44;
	const float ScreenPixelRadiusWorld = DepthScale * RadiusScreen;

	// Clamp the minimum here since our shadows sadly don't actually have infinite resolution near the camera...
	float ShadowResolutionScale = max(0.1f, ScreenPixelRadiusWorld * exp2(ProjectionData.ResolutionLodBias));

	return ShadowResolutionScale;
}

float VirtualShadowMapGetClampedRayDistanceScaleLocal(float CosTheta)
{
	// The further afield our rays go the poorer our approximation is as the "bend" due to our testing
	// against a shadow map instead of along the ray increases. Thus we avoid going all the way to the light
	// where the projection becomes extreme.

	// This clamping function is a bit arbitrary but is fairly smooth and avoids the worst artifacts in practice
	float SinTheta = sqrt(1.0f - CosTheta * CosTheta);
	return 0.75f * saturate(1.5f / (CosTheta + VirtualShadowMap.SMRTCotMaxRayAngleFromLight * SinTheta));
}

FVirtualShadowMapSampleResult TraceLocalLight(
	int VirtualShadowMapId,
	FLightShaderParameters Light,
	uint2 PixelPos,
	const float SceneDepth,
	float3 TranslatedWorldPosition,
	float RayStartOffset,
	const float Noise,
	float3 WorldNormal,
	const FSMRTTraceSettings Settings = GetSMRTTraceSettingsLocal())
{
	FVirtualShadowMapHandle VirtualShadowMapHandle = FVirtualShadowMapHandle::MakeFromId(VirtualShadowMapId);
	const uint LightType = Light.SpotAngles.x > -2.0f ? LIGHT_TYPE_SPOT : LIGHT_TYPE_POINT;
	
	const float3 ToLight = Light.TranslatedWorldPosition - TranslatedWorldPosition;
	const float3 ConeAxis = normalize(ToLight);
	const float DistToLight = length(ToLight);
	const float ConeSin = Light.SourceRadius / DistToLight;

	FVirtualShadowMapSampleResult Result = InitVirtualShadowMapSampleResult();
	Result.bValid = true;
	Result.ShadowFactor = 0.0f;

	FVirtualShadowMapHandle OffsetVirtualShadowMapHandle = VirtualShadowMapHandle;
	if (LightType == LIGHT_TYPE_POINT)
	{
		OffsetVirtualShadowMapHandle = OffsetVirtualShadowMapHandle.MakeOffset(VirtualShadowMapGetCubeFace(-ToLight));
	}

	// Move from PrimaryView translated world to Shadow translated world
	// NOTE: Cube map faces all share the same PreViewTranslation and projection depth scale
	FVirtualShadowMapProjectionShaderData ProjectionData = GetVirtualShadowMapProjectionData(OffsetVirtualShadowMapHandle);
	const float3 PrimaryToShadowTranslation = DFFastLocalSubtractDemote(ProjectionData.PreViewTranslation, PrimaryView.PreViewTranslation);
	const float3 ShadowTranslatedWorld = TranslatedWorldPosition + PrimaryToShadowTranslation;

	const float2 DepthSlopeUV = ComputeDepthSlopeLocalUV(OffsetVirtualShadowMapHandle, ShadowTranslatedWorld, WorldNormal);

	const float ShadowResolutionScale = ComputeShadowResolutionScale(ProjectionData, SceneDepth);
	// Rough scaling from world units -> post projection depth.
	// We use a radially symmetric mapping here instead of the true function to minimize issues crossing cubemap faces.
	const float ShadowDepthFunctionScale = abs(ProjectionData.ShadowViewToClipMatrix._43 / length2(ToLight));

	const float DitherScale = (Settings.TexelDitherScale * ProjectionData.TexelDitherScale) * ShadowResolutionScale;
	const float MaxDitherSlope = VirtualShadowMap.SMRTMaxSlopeBiasLocal * DitherScale;
	const float MaxDepthBias = VirtualShadowMap.SMRTMaxSlopeBiasLocal * ShadowResolutionScale * ShadowDepthFunctionScale;

	// Moving the ray origin means we need to attempt to account for local receiver geometry to avoid
	// incorrect self-shadowing. To this end we move the point on a plane aligned with the shading normal.
	// It would be much better to use geometric (faceted) normals - as with optimal bias - but we do not
	// have access to those. Shading normals that stray too far from the real geometry will cause artifacts.
	const float3x3 ConeTangentToWorldMatrix = GetTangentBasis(ConeAxis);
	const float3 NormalPlaneTangent = mul(ConeTangentToWorldMatrix, WorldNormal);
	const float2 DepthSlopeTangent = -NormalPlaneTangent.xy / NormalPlaneTangent.z;

	// If source angle = 0, we don't need multiple samples on the same ray
	uint SamplesPerRay = Settings.SamplesPerRay;
	if (ConeSin == 0.0f)
	{
		SamplesPerRay = 0;
	}

	// When using receiver mask, we also must compute the base mip level to use
	// We cannot use greedy mip selection in this case since the higher res mips may not have the rendered
	// geometry that covers the requested mask at this receiver. We must use a similar-or-lower mip than
	// the one that we marked.
	// TODO: Any significant mip changes in GetMipLevelLocal should probably affect the above
	// bias/texel dither math as well, which currently assumes a roughly proportional pixel:texel mapping
	uint MipLevel = VirtualShadowMapGetMipLevelLocalFromReceiver(ProjectionData, ShadowTranslatedWorld, SceneDepth);

	uint i = 0;
	uint RayMissCount = 0;
	float StepOffset = Noise;
	float OccluderDistanceSum = 0.0f;
	float MaxOccluderDistance = -1.0f;
	const uint MaxRayCount = Settings.RayCount;
	for( ; i < MaxRayCount; i++ )
	{
		float4 RandSample = VirtualShadowMapGetRandomSample(PixelPos, View.StateFrameIndex, i, MaxRayCount);
			
		float2 LightUV = UniformSampleDiskConcentricApprox( RandSample.xy ).xy;
		LightUV *= ConeSin;
		float SinTheta2 = dot(LightUV, LightUV);
		float CosTheta = sqrt(1.0f - SinTheta2);
		float3 Dir = mul(float3(LightUV, CosTheta), ConeTangentToWorldMatrix);

		float3 RayStartWithDither = ShadowTranslatedWorld;

		if (DitherScale > 0.0f)
		{
			// Offset following the plane of the receiver
			float2 RandomOffset = (RandSample.zw - 0.5f) * DitherScale;
			float OffsetZ = min(MaxDitherSlope, 2.0f * max(0.0f, dot(DepthSlopeTangent, RandomOffset)));
			float3 Offset = mul(float3(RandomOffset, OffsetZ), ConeTangentToWorldMatrix);
			RayStartWithDither += Offset;
		}

		float ClampedEnd = DistToLight * VirtualShadowMapGetClampedRayDistanceScaleLocal(CosTheta);
		float Start = min(RayStartOffset, ClampedEnd - 1e-6f);

		float3 RayStart = RayStartWithDither + Dir * Start;
		float3 RayEnd = RayStartWithDither + Dir * ClampedEnd;

		float DepthBias = 0.0f;
		if (MaxDepthBias > 0.0f)
		{
			// Clamp depth bias to avoid excessive degenerate slope biases causing flickering lit pixels
			DepthBias = min(MaxDepthBias, ComputeOptimalSlopeBiasLocal(OffsetVirtualShadowMapHandle, RayStart, DepthSlopeUV));
		}

		FSMRTResult SMRTResult;
		FSMRTSingleRayState RayState;	// For debug output
		if ( LightType == LIGHT_TYPE_SPOT )
		{
			SMRTResult = ShadowRayCastSpotLight(
				VirtualShadowMapHandle,
				RayStart,
				RayEnd,
				DepthBias,
				MipLevel,
				SamplesPerRay,
				StepOffset,
				Settings.ExtrapolateMaxSlope,
				RayState);
		}
		else
		{
			SMRTResult = ShadowRayCastPointLight(
				VirtualShadowMapHandle,
				RayStart,
				RayEnd,
				DepthBias,
				MipLevel,
				SamplesPerRay,
				StepOffset,
				Settings.ExtrapolateMaxSlope,
				RayState);
		}

		if (SMRTResult.bValidHit)
		{
			// TODO: Do we want to mess with this at all due to our offset?
			float ReceiverDistance = length(RayStart);
			float OccluderDistance = ComputeOccluderDistancePerspective(
				// Re-fetch this here to avoid reg pressure
				GetVirtualShadowMapProjectionData(RayState.VirtualShadowMapHandle).ShadowViewToClipMatrix,
				SMRTResult.HitDepth,
				saturate(RayState.RayStartUVz.z),
				ReceiverDistance);

			OccluderDistanceSum += OccluderDistance;
			MaxOccluderDistance = max(MaxOccluderDistance, OccluderDistance);
		}
		else
		{
			++RayMissCount;
		}
		
		// Debug output (DCE'd if not used)
		Result.ClipmapOrMipLevel = RayState.MipLevel;
		Result.VirtualTexelAddress = RayState.VirtualTexelAddress;
		Result.PhysicalTexelAddress = RayState.PhysicalTexelAddress;

		if (MaxRayCount > 1 && Settings.AdaptiveRayCount > 0)
		{
			// TODO: Adapt this heuristic based on SMRTAdaptiveRayCount as well?
			if( i == 0 )
			{
				bool bHit = SMRTResult.bValidHit;

				// All lanes missed
				bool bAllLanesMiss = WaveActiveAllTrue( !bHit );
				if( bAllLanesMiss )
				{
					break;
				}
			}
			else if( i >= Settings.AdaptiveRayCount)
			{
				// After N iterations and all have hit, assume umbra
				bool bAllLanesHit = WaveActiveAllTrue( RayMissCount == 0 );
				if( bAllLanesHit )
				{
					break;
				}
			}
		}

		uint Seed = asuint( StepOffset );
		StepOffset = ( EvolveSobolSeed( Seed ) >> 8 ) * 5.96046447754e-08; // * 2^-24;
	}
	uint RayCount = min(i + 1, MaxRayCount);		// break vs regular for loop exit

	float OccluderDistance = (OccluderDistanceSum / float(max(1, RayCount - RayMissCount)));
	//OccluderDistance = MaxOccluderDistance;

	Result.ShadowFactor = float(RayMissCount) / float(RayCount);
	Result.RayCount = RayCount;
	Result.OccluderDistance = OccluderDistance;

	return Result;
}

// Generate ray based on light source geometry
bool GenerateRayLocalLight(
	FLightShaderParameters Light,
	uint2 PixelPos,
	float3 TranslatedWorldPosition,
	float3 WorldNormal,
	uint RayIndex,
	uint RayCount,
	inout float3 OutRayStart,
	inout float3 OutRayEnd)
{
	const float3 ToLight = Light.TranslatedWorldPosition - TranslatedWorldPosition;
	const float3 ConeAxis = normalize(ToLight);
	const float DistToLight = length(ToLight);
	const float ConeSin = Light.SourceRadius / DistToLight;

	bool bBackface = IsBackfaceToLocalLight(ToLight, WorldNormal, Light.SourceRadius);	
	if (!bBackface)
	{
		float2 E = VirtualShadowMapGetRandomSample(PixelPos, View.StateFrameIndex, RayIndex, RayCount).xy;
		float2 LightUV = UniformSampleDiskConcentricApprox(E);
		LightUV *= ConeSin;
		float SinTheta2 = dot(LightUV, LightUV);
		float CosTheta = sqrt(1 - SinTheta2);
		float3 Dir = TangentToWorld(float3(LightUV, CosTheta), ConeAxis);

		float ClampedLength = DistToLight * VirtualShadowMapGetClampedRayDistanceScaleLocal(CosTheta);

		OutRayStart = TranslatedWorldPosition;
		OutRayEnd = TranslatedWorldPosition + Dir * ClampedLength;
	}
	return !bBackface;
}
