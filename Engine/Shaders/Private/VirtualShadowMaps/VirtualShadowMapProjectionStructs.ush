// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../LargeWorldCoordinates.ush"
#include "../ByteBuffer.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "VirtualShadowMapHandle.ush"

#define VSM_PSD_OFFSET_RESOLUTION_LOD_BIAS 252 // todo: should use structuredbuffer or improvised offsetof instead
#define VSM_PSD_OFFSET_LIGHT_TYPE 220

// Rougly the same structure as in VirtualShadowMapArray.h, with some logic around LWC values
struct FVirtualShadowMapProjectionShaderData
{
	float4x4 ShadowViewToClipMatrix;
	float4x4 TranslatedWorldToShadowUVMatrix;
	float4x4 ShadowUVToTranslatedWorldMatrix;
	float4x4 TranslatedWorldToShadowUVNormalMatrix;

	float3 LightDirection;
	uint LightType;		// Matches ELightComponentType via defines in SSDDefinitions.ush

	FDFVector3 PreViewTranslation;
	float LightRadius;
	float ResolutionLodBias;

	float3 ClipmapWorldOriginOffset; // Clipmap origin, in shadow translated world

	int2 ClipmapCornerRelativeOffset;
	int ClipmapLevel;					// "Absolute" level, can be negative
	int ClipmapLevelCountRemaining;		// Remaining levels, relative to this one
	uint ClipmapLevelIndex;				// Level index, relative to the base level

	uint Flags;
	float ClipmapLevelWPODistanceDisabledThresholdSquared;
	float LightSourceRadius;
	float TexelDitherScale; // Per-light additional dither filtering
	uint MinMipLevel; // Minimum mip level that is used for this mip chain (only valid for local lights)

	int SceneRendererPrimaryViewId;
	int PersistentViewId;

	int LightId;

	float2 DynamicDepthCullRange;

	// Derived data for convenience when passing the structure around
	FVirtualShadowMapHandle VirtualShadowMapHandle;
	bool bUnCached; // see VSM_PROJ_FLAG_UNCACHED
	bool bUnreferenced; // see VSM_PROJ_FLAG_UNREFERENCED
	bool bIsCoarseClipLevel; // see VSM_PROJ_FLAG_IS_COARSE_CLIP_LEVEL
	bool bHasPrefilteredDistant; // see VSM_PROJ_FLAG_HAS_PREFILTERED_DISTANT;
	bool bUseReceiverMask; // see VSM_PROJ_FLAG_USE_RECEIVER_MASK
	bool bForceCacheDynamicCoarse; // see VSM_PROJ_FLAG_FORCE_CACHE_DYNAMIC_COARSE
};

FVirtualShadowMapProjectionShaderData DecodeVirtualShadowMapProjectionData(ByteAddressBuffer ProjectionData, FVirtualShadowMapHandle VirtualShadowMapHandle)
{
	FVirtualShadowMapProjectionShaderData Result;
	Result.VirtualShadowMapHandle = VirtualShadowMapHandle;

	const uint VSMOffset = VirtualShadowMapHandle.GetDataIndex() * VSM_PSD_STRIDE;
	uint CurrentLoadOffset = VSMOffset;
	LoadAndIncrementOffset(Result.ShadowViewToClipMatrix, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.TranslatedWorldToShadowUVMatrix, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.ShadowUVToTranslatedWorldMatrix, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.TranslatedWorldToShadowUVNormalMatrix, ProjectionData, CurrentLoadOffset);

	// NOTE: Stick with struct-element-sized loads for the moment since we may only be using subsets
	// of the data in the calling code and we want to ensure the compiler has simple DCE options.
	LoadAndIncrementOffset(Result.LightDirection, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.LightType, ProjectionData, CurrentLoadOffset);
	checkSlow(VSM_PSD_OFFSET_LIGHT_TYPE == (CurrentLoadOffset - VSMOffset));
		 
	float3 PreViewTranslationHigh;
	LoadAndIncrementOffset(PreViewTranslationHigh, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.LightRadius, ProjectionData, CurrentLoadOffset);

	float3 PreViewTranslationLow;
	LoadAndIncrementOffset(PreViewTranslationLow, ProjectionData, CurrentLoadOffset);
	checkSlow(VSM_PSD_OFFSET_RESOLUTION_LOD_BIAS == (CurrentLoadOffset - VSMOffset));
	LoadAndIncrementOffset(Result.ResolutionLodBias, ProjectionData, CurrentLoadOffset);

	LoadAndIncrementOffset(Result.ClipmapWorldOriginOffset, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.LightSourceRadius, ProjectionData, CurrentLoadOffset);
	
	LoadAndIncrementOffset(Result.ClipmapCornerRelativeOffset, ProjectionData, CurrentLoadOffset);

	uint ClipmapLevel_ClipmapLevelCountRemaining = 0;
	LoadAndIncrementOffset(ClipmapLevel_ClipmapLevelCountRemaining, ProjectionData, CurrentLoadOffset);
	if (int(ClipmapLevel_ClipmapLevelCountRemaining) != -1)
	{
		Result.ClipmapLevel						= int(ClipmapLevel_ClipmapLevelCountRemaining >> 16u) - VSM_PACKED_CLIP_LEVEL_BIAS;
		Result.ClipmapLevelCountRemaining		= int(ClipmapLevel_ClipmapLevelCountRemaining & 0xFFu);
		Result.ClipmapLevelIndex				= (ClipmapLevel_ClipmapLevelCountRemaining >> 8u) & 0xFFu;
	}
	else
	{
		Result.ClipmapLevel						= VSM_PACKED_CLIP_LEVEL_BIAS;
		Result.ClipmapLevelCountRemaining		= -1;
		Result.ClipmapLevelIndex				= 0;
	}
	uint PackedCullingViewId;
	LoadAndIncrementOffset(PackedCullingViewId, ProjectionData, CurrentLoadOffset);

	Result.SceneRendererPrimaryViewId = int(PackedCullingViewId >> 16u) - 1;
	Result.PersistentViewId = int(PackedCullingViewId & 0xFFFFu) - 1;

	LoadAndIncrementOffset(Result.Flags, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.ClipmapLevelWPODistanceDisabledThresholdSquared, ProjectionData, CurrentLoadOffset);
	
	LoadAndIncrementOffset(Result.TexelDitherScale, ProjectionData, CurrentLoadOffset);
	LoadAndIncrementOffset(Result.MinMipLevel, ProjectionData, CurrentLoadOffset);

	LoadAndIncrementOffset(Result.LightId, ProjectionData, CurrentLoadOffset);

	LoadAndIncrementOffset(Result.DynamicDepthCullRange, ProjectionData, CurrentLoadOffset);

	// Computed data
	Result.PreViewTranslation				= MakeDFVector3(PreViewTranslationHigh, PreViewTranslationLow);

	Result.bUnCached = (Result.Flags & VSM_PROJ_FLAG_UNCACHED) != 0u;
	Result.bUnreferenced = (Result.Flags & VSM_PROJ_FLAG_UNREFERENCED) != 0u;
	Result.bIsCoarseClipLevel = (Result.Flags & VSM_PROJ_FLAG_IS_COARSE_CLIP_LEVEL) != 0u;
	Result.bHasPrefilteredDistant = (Result.Flags & VSM_PROJ_FLAG_HAS_PREFILTERED_DISTANT) != 0U;
	Result.bUseReceiverMask = (Result.Flags & VSM_PROJ_FLAG_USE_RECEIVER_MASK) != 0u;
	Result.bForceCacheDynamicCoarse = (Result.Flags & VSM_PROJ_FLAG_FORCE_CACHE_DYNAMIC_COARSE) != 0;
	
	return Result;
}

FVirtualShadowMapProjectionShaderData GetVirtualShadowMapProjectionData(FVirtualShadowMapHandle VirtualShadowMapHandle)
{
	return DecodeVirtualShadowMapProjectionData(VirtualShadowMap.ProjectionData, VirtualShadowMapHandle);
}

FVirtualShadowMapProjectionShaderData GetVirtualShadowMapProjectionData(FVirtualShadowMapHandle VirtualShadowMapHandle, ByteAddressBuffer ProjectionDataBuffer)
{
	return DecodeVirtualShadowMapProjectionData(ProjectionDataBuffer, VirtualShadowMapHandle);
}
