// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapSMRTCommon.ush:
=============================================================================*/

#pragma once

//#include "../PathTracing/Utilities/PathTracingRandomSequence.ush"
#include "../BlueNoise.ush"

struct FSMRTSample
{
	bool bValid;	
	float SampleDepth;
	float ReferenceDepth;
	float ExtrapolateSlope;
	bool bResetExtrapolation;
};

FSMRTSample InitSMRTSample()
{
	FSMRTSample Result;
	Result.bValid = false;
	Result.SampleDepth = 0;
	Result.ReferenceDepth = 0;
	Result.ExtrapolateSlope = 0;
	Result.bResetExtrapolation = false;
	return Result;
}

struct FSMRTResult
{
	bool bValidHit;
	float HitDepth;	// Valid if bValidHit is true
};

// See http://extremelearning.com.au/unreasonable-effectiveness-of-quasirandom-sequences/
float2 R2Sequence(int n)
{
	return frac(float(n) * float2(0.754877669f, 0.569840296f));
}

float4 VirtualShadowMapGetRandomSample(
	uint2 PixelPos,
	uint TimeIndex,
	uint SampleIndex,
	uint MaxSampleCount)
{
#if 1
	{
		// Blue noise is cheaper to compute
		int2 Offset1 = int2(R2Sequence(SampleIndex) * BlueNoise.Dimensions.xy);
		int2 Offset2 = int2(R2Sequence(SampleIndex + MaxSampleCount) * BlueNoise.Dimensions.xy);
		float4 RandomSample;
		RandomSample.xy = BlueNoiseVec2(PixelPos + Offset1, TimeIndex);
		RandomSample.zw = BlueNoiseVec2(PixelPos + Offset2, TimeIndex);
		return RandomSample;
	}
#else
	{
		return RandomSequenceCreate(uint3(PixelPos, TimeIndex), SampleIndex, MaxSampleCount).Get4D();
	}
#endif
}

// Handy structure for locally modifying some common SMRT parameters
struct FSMRTTraceSettings
{
	int AdaptiveRayCount;
	int RayCount;
	int SamplesPerRay;
	float ExtrapolateMaxSlope;
	float TexelDitherScale;
};

FSMRTTraceSettings GetSMRTTraceSettingsDirectional()
{
	FSMRTTraceSettings Settings;
	Settings.AdaptiveRayCount		= VirtualShadowMap.SMRTAdaptiveRayCount;
	Settings.RayCount				= VirtualShadowMap.SMRTRayCountDirectional;
	Settings.SamplesPerRay			= VirtualShadowMap.SMRTSamplesPerRayDirectional;
	Settings.ExtrapolateMaxSlope	= VirtualShadowMap.SMRTExtrapolateMaxSlopeDirectional;
	Settings.TexelDitherScale		= VirtualShadowMap.SMRTTexelDitherScaleDirectional;
	return Settings;
}

FSMRTTraceSettings GetSMRTTraceSettingsLocal()
{
	FSMRTTraceSettings Settings;
	Settings.AdaptiveRayCount		= VirtualShadowMap.SMRTAdaptiveRayCount;
	Settings.RayCount				= VirtualShadowMap.SMRTRayCountLocal;
	Settings.SamplesPerRay			= VirtualShadowMap.SMRTSamplesPerRayLocal;
	Settings.ExtrapolateMaxSlope	= VirtualShadowMap.SMRTExtrapolateMaxSlopeLocal;
	Settings.TexelDitherScale		= VirtualShadowMap.SMRTTexelDitherScaleLocal;
	return Settings;
}
