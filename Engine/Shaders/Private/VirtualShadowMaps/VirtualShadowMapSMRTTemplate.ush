// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMapSMRTTemplate.ush:

Required parameters before including this file:
1) Include "SMRTCommon.ush"
2) Define a "ray state" structure (FRayState here, but must be unique)
3) Implement FSMRTSample SMRTFindSample(inout FRayState RayState, float SampleTime)
  - NOTE: ReferenceDepth must always be set, even if the sample is invalid
4) Include this file:
#define SMRT_TEMPLATE_RAY_STRUCT FRayState
#include "VirtualShadowMapSMRTTemplate.ush"
#undef SMRT_TEMPLATE_RAY_STRUCT
=============================================================================*/

// NOTE: No pragma(once) or inclusion guards because this file is meant to be
// included multiple times as our hacky way to generate template-like instantiations
// before HLSL supports templates natively.

// If defined, unrolls the loop and ignores the NumSteps parameter in favor of the define
#ifndef SMRT_TEMPLATE_STATIC_SAMPLES_PER_RAY
#define SMRT_TEMPLATE_STATIC_SAMPLES_PER_RAY -1
#endif

FSMRTResult SMRTRayCast(
	inout SMRT_TEMPLATE_RAY_STRUCT RayState,
	int NumSteps,
	float StepOffset)
{
	// NOTE: Need a magic initializer that we can detect. Would be natural to use DepthHistoryTime for this,
	// but we want that register to disappear (DCE) when SMRT_EXTRAPOLATE_SLOPE is disabled.
	// DepthHistory *can* be negative though when crossing clipmap boundaries, as the depth range expands.
	// In practice it will only ever be down to -2 or so.
	const float DepthHistoryNotSet = -10000.0f;
	float DepthHistory = DepthHistoryNotSet;

	float DepthHistoryTime = -1.0f;
	float DepthSlope = 0;

	const float TimeScale = -1.0f / NumSteps;
	const float TimeBias = 1.0f + ( 1.0 - StepOffset ) * TimeScale;

	// This doesn't get used in the first iteration of the loop and so will always
	// be valid by the time that it does.
	float PrevReferenceDepth = -1;

	bool bValidHit =  false;
	float SlopeScale = 1;
#if UBERSHADER
	NumSteps = SMRTSamplePerRay;
	SlopeScale = bSMRTExtrapolateSlope ? 1 : 0;
#elif SMRT_TEMPLATE_STATIC_SAMPLES_PER_RAY >= 0
	NumSteps = SMRT_TEMPLATE_STATIC_SAMPLES_PER_RAY;
	UNROLL
#endif
	for (int i = 0; i <= NumSteps; i++)
	{
		const float SampleTime = ( i == NumSteps ) ? 0 : Pow2( TimeScale * i + TimeBias );

		FSMRTSample Sample = SMRTFindSample(RayState, SampleTime);
		const float ReferenceDepth = Sample.ReferenceDepth;

		if (Sample.bResetExtrapolation)
		{
			DepthSlope = Sample.ExtrapolateSlope;
		}

		if (Sample.bValid)
		{
			const float SampleDepth = Sample.SampleDepth;
			if (DepthHistory == DepthHistoryNotSet)
			{
				// First valid sample we've seen. Do a regular depth compare.
				DepthHistory = SampleDepth;
				DepthHistoryTime = SampleTime;
				if (SampleDepth > ReferenceDepth)
				{
					FSMRTResult Result;
					Result.bValidHit = true;
					Result.HitDepth = SampleDepth;
					return Result;
				}
			}
			else
			{
				const float DeltaReferenceDepth = ReferenceDepth - PrevReferenceDepth;

				// Add a small relative error to the comparison to avoid missing surfaces due to numeric precision issues
				// Without this there are occasionally flickering fireflies in fully shadowed regions with SMRT
				const float EpsScale = 1.05f;
				const float CompareTolerance = abs(DeltaReferenceDepth) * EpsScale;
								
				const bool bBehind = (SampleDepth - ReferenceDepth) > CompareTolerance;
				float DepthForComparison = SampleDepth;

				float DeltaHistoryTime = SampleTime - DepthHistoryTime;

				if (bBehind)
				{
					#if SMRT_EXTRAPOLATE_SLOPE
						DepthForComparison = SlopeScale * DepthSlope * DeltaHistoryTime + DepthHistory;
					#else
						DepthForComparison = DepthHistory;
					#endif
				}
				else
				{
					if (SampleDepth != DepthHistory)
					{
						// NOTE: DCE will remove this if SMRT_EXTRAPOLATE_WITH_SLOPE is false
						const float SlopeClamp = Sample.ExtrapolateSlope;
						DepthSlope = (SampleDepth - DepthHistory) / DeltaHistoryTime;
						DepthSlope = clamp(DepthSlope, -SlopeClamp, SlopeClamp);

						DepthHistory = SampleDepth;
						DepthHistoryTime = SampleTime;
					}
				}

				float DepthDiff = ReferenceDepth - DepthForComparison;
				float HalfCompareTolerance = 0.5 * CompareTolerance;
				bool bHit = abs(DepthDiff + HalfCompareTolerance) < HalfCompareTolerance;
				if (bHit)
				{
					FSMRTResult Result;
					Result.bValidHit = true;
					Result.HitDepth = DepthForComparison;
					return Result;
				}
			}

			PrevReferenceDepth = ReferenceDepth;
		}
	}

	FSMRTResult Result;
	Result.bValidHit = false;
	Result.HitDepth = -1.0f;
	return Result;
}
