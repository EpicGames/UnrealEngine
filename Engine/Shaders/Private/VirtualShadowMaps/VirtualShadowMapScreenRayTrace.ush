// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"

#ifndef SCREEN_RAY_SAMPLES
#define SCREEN_RAY_SAMPLES 4
#endif

// Screen space ray trace to attempt to skip over ambiguous regions near the receiver surface
// Returns length at which to start the virtual shadow map ray; usually this is where the screen ray ended or went behind a surface
float VirtualShadowMapScreenRayCast(
	float3 RayOriginTranslatedWorld,
	float3 RayDirection,
	float RayLength,
	float Dither)
{
	float4 RayStartClip	= mul(float4(RayOriginTranslatedWorld, 1), View.TranslatedWorldToClip);
	float4 RayDirClip	= mul(float4(RayDirection * RayLength, 0), View.TranslatedWorldToClip);
	float4 RayEndClip	= RayStartClip + RayDirClip;

	float3 RayStartScreen = RayStartClip.xyz / RayStartClip.w;
	float3 RayEndScreen = RayEndClip.xyz / RayEndClip.w;

	float3 RayStepScreen = RayEndScreen - RayStartScreen;

	float3 RayStartUVz = float3(RayStartScreen.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz, RayStartScreen.z);
	float3 RayStepUVz  = float3(RayStepScreen.xy * View.ScreenPositionScaleBias.xy, RayStepScreen.z);

	float4 RayDepthClip	= RayStartClip + mul(float4(0, 0, RayLength, 0), View.ViewToClip);
	float3 RayDepthScreen = RayDepthClip.xyz / RayDepthClip.w;

	const int Steps = SCREEN_RAY_SAMPLES;
	float StepOffset = Dither - 0.5f;
	const float Step = 1.0 / Steps;
	float SampleTime = StepOffset * Step + Step;

	const float StartDepth = SceneTexturesStruct.SceneDepthTexture.SampleLevel(SceneTexturesStruct_SceneDepthTextureSampler, RayStartUVz.xy, 0).r;

	UNROLL
	for (int i = 0; i < Steps; i++)
	{
		float3 SampleUVz = RayStartUVz + RayStepUVz * SampleTime;
		float SampleDepth = SceneTexturesStruct.SceneDepthTexture.SampleLevel(SceneTexturesStruct_SceneDepthTextureSampler, SampleUVz.xy, 0).r;

		// Avoid self-intersection with the start pixel (exact comparison due to point sampling depth buffer)
		if (SampleDepth != StartDepth)
		{
			if (SampleUVz.z < SampleDepth)
			{
				// Behind geometry. Back up a bit along the ray and do the VSM sample from there.
				return RayLength * max(0.0, SampleTime - 1.5f * Step);
			}
		}

		SampleTime += Step;
	}

	// Got to the end of the ray without going behind or hitting anything
	return RayLength;
}
