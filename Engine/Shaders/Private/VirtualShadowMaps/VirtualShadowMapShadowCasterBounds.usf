// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"

#define NANITE_USE_VIEW_UNIFORM_BUFFER 1
#include "VirtualShadowMapVisualize.ush"

#ifndef MSAA_SAMPLE_COUNT
#define MSAA_SAMPLE_COUNT 1
#endif

struct FVSToPS
{
	float4 SvPosition : SV_POSITION;
	float3 CubePosition : DBG_CUBE_POSITION;
	float3 WorldPosition : DBG_WORLD_POSITION;
	nointerpolation float3 Color : DBG_COLOR;
	nointerpolation uint InstanceId : DBG_INSTANCEID;
	nointerpolation float3 BoundingBoxMin : DBG_BB_MIN;
};

float3 GetBoundsVertexTWSPosition(float3 CubeVertex, in FInstanceSceneData InstanceData)
{
	float3 LocalPosition = (CubeVertex * InstanceData.LocalBoundsExtent * InstanceData.DeterminantSign) + InstanceData.LocalBoundsCenter;
	float4 WorldPosition = DFTransformLocalToTranslatedWorld(LocalPosition, InstanceData.LocalToWorld, ResolvedView.PreViewTranslation);
	return WorldPosition.xyz;
}

void MainVS(
	float3 InPosition : ATTRIBUTE0,
	uint InstanceId : SV_InstanceID,
	out FVSToPS Output
)
{
	ResolvedView = ResolveView();

	Output.SvPosition = asfloat(0xFFFFFFFF);
	Output.CubePosition = InPosition;
	Output.WorldPosition = asfloat(0xFFFFFFFF);
	Output.Color = 0;
	Output.InstanceId = InstanceId;
	Output.BoundingBoxMin = 0;

	FInstanceSceneData InstanceData				= GetInstanceSceneData(InstanceId);
	if (!InstanceData.ValidInstance)
	{
		return;
	}

	FNaniteView NaniteView						= GetNaniteView(ResolvedView.GPUSceneViewId);
	FInstanceViewData InstanceViewData			= GetInstanceViewData(InstanceId, ResolvedView.GPUSceneViewId);
	FInstanceDynamicData InstanceDynamicData	= CalculateInstanceDynamicData(NaniteView, InstanceData, true);
	FPrimitiveSceneData PrimitiveData			= GetPrimitiveData(InstanceData.PrimitiveId);

	FShadowCasterDrawParameters DrawParameters = GetShadowCasterDrawParameters(NaniteView, InstanceData, InstanceViewData, InstanceDynamicData, PrimitiveData);

	if (DrawParameters.bShowBounds)
	{
		Output.WorldPosition = GetBoundsVertexTWSPosition(InPosition, InstanceData);
		Output.SvPosition = mul(float4(Output.WorldPosition, 1), ResolvedView.TranslatedWorldToClip);
		Output.Color = DrawParameters.Color;
		Output.BoundingBoxMin = GetBoundsVertexTWSPosition(float3(-1, -1, -1), InstanceData);
	}
}

#if MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, MSAA_SAMPLE_COUNT> DebugDepth;
#else
Texture2D<float> DebugDepth;
#endif

float4 SampleOffsetArray[MSAA_SAMPLE_COUNT];

Texture2D<float> SceneDepth;

float ComputeDepthMask(float DeviceZA, float DeviceZB, float Fade = 1e-5)
{
	return saturate(1.0f - (DeviceZA - DeviceZB) / Fade);
}

// Find the axis with the largest absolute value, and return a vector pointing to that axis (or its negation)
float3 FindMajorComponent(float3 InPosition)
{
	float3 A = abs(InPosition);

	uint Index = (A.y > A.x) ? 1 : 0;
	uint IndexOfMaxComponent = (A.z > A[Index]) ? 2 : Index;

	float3 Result = 0;
	Result[IndexOfMaxComponent] = InPosition[IndexOfMaxComponent] > 0 ? 1 : -1;
	return Result;
}

// Given a position on a cube, return the point after projection onto the nearest cube edge.
float3 ProjectOntoNearestCubeEdge(float3 InPosition)
{
	float3 UnitLocalPos = InPosition * 0.5 + 0.5;
	float3 AbsLocalPos = abs(InPosition);

	float3 Edge = 0;
	if (AbsLocalPos.x < AbsLocalPos.y && AbsLocalPos.x < AbsLocalPos.z) { Edge = float3(UnitLocalPos.x, round(UnitLocalPos.y), round(UnitLocalPos.z)); }
	else if (AbsLocalPos.y < AbsLocalPos.x && AbsLocalPos.y < AbsLocalPos.z) { Edge = float3(round(UnitLocalPos.x), UnitLocalPos.y, round(UnitLocalPos.z)); }
	else /*if (AbsLocalPos.z > AbsLocalPos.x && AbsLocalPos.z > AbsLocalPos.y)*/ { Edge = float3(round(UnitLocalPos.x), round(UnitLocalPos.y), UnitLocalPos.z); }
	Edge = Edge * 2 - 1;
	return Edge;
}

float LinStep(float E0, float E1, float Value)
{
    return clamp((Value - E0) / (E1 - E0), 0.0f, 1.0f);
}

void MainPS(
	FVSToPS Input,
	out float4 OutColor : SV_Target0
)
{
	ResolvedView = ResolveView();

	FInstanceSceneData InstanceData				= GetInstanceSceneData(Input.InstanceId);
	FNaniteView NaniteView						= GetNaniteView(ResolvedView.GPUSceneViewId);
	FInstanceViewData InstanceViewData			= GetInstanceViewData(Input.InstanceId, ResolvedView.GPUSceneViewId);
	FInstanceDynamicData InstanceDynamicData	= CalculateInstanceDynamicData(NaniteView, InstanceData, true);
	FPrimitiveSceneData PrimitiveData			= GetPrimitiveData(InstanceData.PrimitiveId);

	// Per-sample is technically more correct than per-pixel, but it's more expensive and I can't see the difference, so we just use sample 0.
	uint SampleIndex = 0;

#if MSAA_SAMPLE_COUNT > 1
	float Depth = DebugDepth.Load((int2)Input.SvPosition.xy, SampleIndex);
#else
	float Depth = DebugDepth.Load(int3(Input.SvPosition.xy, 0));
#endif

	float SceneDepthVal = SceneDepth.Load(int3(Input.SvPosition.xy, 0));

	float3 N = normalize(cross(ddx(Input.WorldPosition), ddy(Input.WorldPosition)));
	float3 V = -normalize(Input.WorldPosition - View.TranslatedWorldCameraOrigin);
	float NdotV = dot(N, V);

	float Mask = ComputeDepthMask(Depth, Input.SvPosition.z);
	float SceneMask = ComputeDepthMask(SceneDepthVal, Input.SvPosition.z, 1e-6);

	float EdgeDistance;
	{
		float3 Edge = ProjectOntoNearestCubeEdge(Input.CubePosition);

		float3 EdgeLocalPosition = (Edge * InstanceData.LocalBoundsExtent * InstanceData.DeterminantSign) + InstanceData.LocalBoundsCenter;
		float4 EdgeWorldPosition = DFTransformLocalToTranslatedWorld(EdgeLocalPosition, InstanceData.LocalToWorld, ResolvedView.PreViewTranslation);
		float WorldspaceEdgeDistance = length(EdgeWorldPosition.xyz - Input.WorldPosition);
		EdgeDistance = WorldspaceEdgeDistance / ConvertFromDeviceZ(Input.SvPosition.z);
	}

	float3 Normal = FindMajorComponent(Input.CubePosition);

	float Corner;
	{
		const float CornerFalloff = 0.02;
		const float CornerWidth = 0.0015;
		Corner = LinStep(CornerWidth, CornerWidth * CornerFalloff, EdgeDistance);

		// Hidden if behind scene color
		Corner *= lerp(0.0, 1.0, SceneMask);
		// Translucent if behind other boxes
		Corner *= lerp(0.3, 1.0, Mask);
	}

	float Hatching;
	{
		const float HatchingAlpha = 0.4;
		const float HatchingLineWidth = 0.4;
		const float HatchingFalloff = 0.02;
		// Scale hatching along each axis with object bounds
		float HatchingScale = 5.0 / length(InstanceData.LocalBoundsExtent * (1 - Normal));
		float L1Distance = VectorSum(Input.WorldPosition - Input.BoundingBoxMin);
		Hatching = HatchingAlpha * LinStep(
			(1 - HatchingLineWidth) * (1 - HatchingFalloff), 
			1 - HatchingLineWidth, 
			frac(L1Distance * HatchingScale));

		// Fade out with depth - only visible in nearfield
		Hatching *= 1 - LinStep(3000, 6000, ConvertFromDeviceZ(Input.SvPosition.z));
		// Hidden if behind scene color
		Hatching *= lerp(0.0, 1.0, Mask);
		// Hidden if behind other boxes
		Hatching *= lerp(0.0, 1.0, SceneMask);
		// Apply dot(N,V) alpha
		Hatching *= abs(NdotV) * 0.7 + 0.3;
	}

	float Uniform = 0.13;
	{
		// Hidden if behind scene color
		Uniform *= lerp(0.0, 1.0, SceneMask);
		// Translucent if behind other boxes
		Uniform *= lerp(0.3, 1.0, Mask);
	}

	float3 Color = Input.Color;
	{
		float Luma = VectorSum(Color) / 3;
		// Desaturate if behind other boxes
		Color = lerp(Luma.xxx * 0.1, Color, Mask * 0.7 + 0.3);
		// Apply dot(N,V) shading
		Color *= abs(NdotV) * 0.7 + 0.3;
	}

	float4 Result;
	Result.rgb = Color;
	Result.a = max3(Corner, Uniform, Hatching);

	if (Result.a == 0)
	{
		discard;
	}

	OutColor = Result;
}
