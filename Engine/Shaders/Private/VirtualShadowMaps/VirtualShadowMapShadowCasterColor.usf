// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "../ScreenPass.ush"
#include "../SceneData.ush"
#include "VirtualShadowMapVisualize.ush"

Texture2D<uint> DebugAuxTexture;
Texture2D SceneColorTexture;

SCREEN_PASS_TEXTURE_VIEWPORT(DebugAuxTextureViewport)
SCREEN_PASS_TEXTURE_VIEWPORT(SceneColorTextureViewport)

FScreenTransform ViewportUVToDebugAuxTextureUV;
FScreenTransform ViewportUVToSceneColorTextureUV;

float4 MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0
) : SV_Target0
{
	ResolvedView = ResolveView();
	float2 UV = UVAndScreenPos.xy;
	
	const float2 DebugAuxTextureUV = ApplyScreenTransform(UV, ViewportUVToDebugAuxTextureUV);
	const int2 DebugAuxPixelPosition = int2(DebugAuxTextureUV * DebugAuxTextureViewport_Extent);

	const float2 SceneColorTextureUV = ApplyScreenTransform(UV, ViewportUVToSceneColorTextureUV);
	const int2 SceneColorPixelPosition = int2(SceneColorTextureUV * SceneColorTextureViewport_Extent);

	uint InstanceId = DebugAuxTexture.Load(int3(DebugAuxPixelPosition, 0));
	float Shading = SceneColorTexture.Load(int3(SceneColorPixelPosition, 0)).r; // Assumes DebugViewModePixelShader outputs dot(N,V) shading

	if (InstanceId == 0xFFFFFFFF)
	{
		discard;
	}
	
	FNaniteView NaniteView						= GetNaniteView(ResolvedView.GPUSceneViewId);
	FInstanceSceneData InstanceData				= GetInstanceSceneData(InstanceId);

	if (!InstanceData.ValidInstance)
	{
		discard;
	}

	FInstanceViewData InstanceViewData			= GetInstanceViewData(InstanceId, NaniteView.SceneRendererPrimaryViewId);
	FInstanceDynamicData InstanceDynamicData	= CalculateInstanceDynamicData(NaniteView, InstanceData, true);
	FPrimitiveSceneData PrimitiveData			= GetPrimitiveData(InstanceData.PrimitiveId);

	FShadowCasterDrawParameters DrawParameters = GetShadowCasterDrawParameters(NaniteView, InstanceData, InstanceViewData, InstanceDynamicData, PrimitiveData);

	return float4(DrawParameters.Color * Shading, 1);
}
