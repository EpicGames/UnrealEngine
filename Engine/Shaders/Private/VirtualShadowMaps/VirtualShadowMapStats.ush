// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualShadowMapStats.ush: 
=============================================================================*/

#include "/Engine/Shared/VirtualShadowMapDefinitions.h"

#include "../Common.ush"
#include "../WaveOpUtil.ush"



#ifndef VSM_GENERATE_STATS
#define VSM_GENERATE_STATS 0
#endif // VSM_GENERATE_STATS

RWStructuredBuffer<uint> OutStatsBuffer;

void StatsBufferWrite(uint Index, uint Value, bool bWriteEnabled = VSM_GENERATE_STATS)
{
	if (bWriteEnabled)
	{
		OutStatsBuffer[Index] = Value;
	}
}

void StatsBufferInterlockedAdd(uint Index, uint Value, bool bWriteEnabled = VSM_GENERATE_STATS)
{
	if (bWriteEnabled)
	{
		InterlockedAdd(OutStatsBuffer[Index], Value);
	}
}

void StatsBufferInterlockedInc(uint Index, bool bWriteEnabled = VSM_GENERATE_STATS)
{
	if (bWriteEnabled)
	{
		WaveInterlockedAddScalar(OutStatsBuffer[Index], 1U);
	}
}

void StatsBufferInterlockedEnableFlags(uint Index, uint Flags, bool bEnabled, bool bWriteEnabled = VSM_GENERATE_STATS)
{
	if (bWriteEnabled)
	{
		if (bEnabled)
		{
			WaveInterlockedOr(OutStatsBuffer[Index], Flags * (uint)bEnabled);
		}
	}
}
