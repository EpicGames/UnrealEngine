// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Common.ush"
#include "VirtualShadowMapHandle.ush"
#include "VirtualShadowMapProjectionStructs.ush"
#include "/Engine/Shared/VirtualShadowMapDefinitions.h"
#include "/Engine/Shared/LightDefinitions.h"

StructuredBuffer<uint> PrevThrottleBuffer;
StructuredBuffer<uint> PrevNanitePerformanceFeedback;

StructuredBuffer<FNextVirtualShadowMapData> NextVirtualShadowMapData;
uint NextVirtualShadowMapDataCount;

uint NumThrottleBufferEntries;
RWStructuredBuffer<uint> OutThrottleBuffer;

RWByteAddressBuffer InOutProjectionData;

float DynResThrottleStrength;
float ThrottleLoadBudget;
float ThrottleEMAHistoryWeight;

float ThrottleMaxBiasDirectional;
float ThrottleMaxBiasLocal;

[numthreads(64, 1, 1)]
void ProcessPrevFramePerfDataCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalClustersHW = PrevNanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_HW_CLUSTERS];
	const uint TotalClustersSW = PrevNanitePerformanceFeedback[VSM_NPF_HEADER_TOTAL_SW_CLUSTERS];

	if (DispatchThreadId == 0)
	{
		OutThrottleBuffer[VSM_TB_HEADER_TOTAL_HW_CLUSTERS] = TotalClustersHW;
		OutThrottleBuffer[VSM_TB_HEADER_TOTAL_SW_CLUSTERS] = TotalClustersSW;
	}

	if(DispatchThreadId < NextVirtualShadowMapDataCount)
	{
		FVirtualShadowMapHandle PrevVSMHandle = FVirtualShadowMapHandle::MakeFromId(DispatchThreadId);
		const FNextVirtualShadowMapData NextVSMData = NextVirtualShadowMapData[PrevVSMHandle.GetDataIndex()];

		if (NextVSMData.Flags & VSM_NEXT_FLAG_ID_VALID)
		{
			FVirtualShadowMapHandle NextVSMHandle = FVirtualShadowMapHandle::MakeFromId(NextVSMData.NextVirtualShadowMapId);

			const uint NPFOffset = VSM_NPF_SIZEOF_HEADER + VSM_NPF_SIZEOF_ENTRY * PrevVSMHandle.GetDataIndex();
			const uint ClustersHW = PrevNanitePerformanceFeedback[NPFOffset + VSM_NPF_ENTRY_HW_CLUSTERS];
			const uint ClustersSW = PrevNanitePerformanceFeedback[NPFOffset + VSM_NPF_ENTRY_SW_CLUSTERS];

			const uint ThrottleBufferOffset = VSM_TB_SIZEOF_HEADER + VSM_TB_SIZEOF_ENTRY * NextVSMHandle.GetDataIndex();
			OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_HW_CLUSTERS] = ClustersHW;
			OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_SW_CLUSTERS] = ClustersSW;
			OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_THROTTLE] = PrevThrottleBuffer[VSM_TB_SIZEOF_HEADER + VSM_TB_SIZEOF_ENTRY * PrevVSMHandle.GetDataIndex() + VSM_TB_ENTRY_THROTTLE];

			InterlockedMax(OutThrottleBuffer[VSM_TB_HEADER_MAX_HW_CLUSTERS], ClustersHW);
			InterlockedMax(OutThrottleBuffer[VSM_TB_HEADER_MAX_SW_CLUSTERS], ClustersSW);
		}
	}
}

float CalculateCostHeuristic(const float NaniteClustersHW, const float NaniteClustersSW)
{
	// todo: can significantly improve the fit of this heuristic with page count data (requested, cached, invalidated)
	return 0.00012176f * NaniteClustersHW + 0.00002700f * NaniteClustersSW;
}

void ApplyResolutionLODBias(RWByteAddressBuffer ProjectionData, FVirtualShadowMapHandle VirtualShadowMapHandle, float ExtraBias)
{
	FVirtualShadowMapProjectionShaderData Data;
	
	// Load
	uint Offset = VirtualShadowMapHandle.GetDataIndex() * VSM_PSD_STRIDE + VSM_PSD_OFFSET_RESOLUTION_LOD_BIAS;
	Data.ResolutionLodBias = asfloat(ProjectionData.Load(Offset));

	// Modify
	Data.ResolutionLodBias += ExtraBias;

	// Store
	ProjectionData.Store(Offset, asuint(Data.ResolutionLodBias));
}

uint GetLightType(RWByteAddressBuffer ProjectionData, FVirtualShadowMapHandle VirtualShadowMapHandle)
{
	uint Offset = VirtualShadowMapHandle.GetDataIndex() * VSM_PSD_STRIDE + VSM_PSD_OFFSET_LIGHT_TYPE;
	return ProjectionData.Load(Offset);
}

[numthreads(64, 1, 1)]
void UpdateThrottleParametersCS(uint DispatchThreadId : SV_DispatchThreadID)
{
	const uint TotalClustersHW = OutThrottleBuffer[VSM_TB_HEADER_TOTAL_HW_CLUSTERS];
	const uint TotalClustersSW = OutThrottleBuffer[VSM_TB_HEADER_TOTAL_SW_CLUSTERS];
	const uint MaxClustersHW = OutThrottleBuffer[VSM_TB_HEADER_MAX_HW_CLUSTERS];
	const uint MaxClustersSW = OutThrottleBuffer[VSM_TB_HEADER_MAX_SW_CLUSTERS];
	
	float GlobalThrottleIntensity;
	{
		float TotalCostHeuristic = CalculateCostHeuristic(TotalClustersHW, TotalClustersSW);
	
		bool bThrottleBasedOnDynRes = DynResThrottleStrength >= 0;
		if (bThrottleBasedOnDynRes)
		{
			GlobalThrottleIntensity = DynResThrottleStrength;
		}
		else
		{
			// hand-tuned parameters
			const float ThrottleResponse = 0.5f;
			const float ThrottleMaxStepUp = 0.2f;
			const float ThrottleMaxStepDown = -0.01f;

			float TotalThrottleHeuristic = TotalCostHeuristic - ThrottleLoadBudget;
			float ThrottleModifier = clamp(TotalThrottleHeuristic * ThrottleResponse, ThrottleMaxStepDown, ThrottleMaxStepUp);
	
			float PrevThrottleIntensity = asfloat(PrevThrottleBuffer[VSM_TB_HEADER_TOTAL_THROTTLE]);
			GlobalThrottleIntensity = saturate(PrevThrottleIntensity + ThrottleModifier);
		}
	}

	if (DispatchThreadId == 0)
	{
		OutThrottleBuffer[VSM_TB_HEADER_TOTAL_THROTTLE] = asuint(GlobalThrottleIntensity);
	}

	if(DispatchThreadId < NumThrottleBufferEntries)
	{
		FVirtualShadowMapHandle VSMHandle = FVirtualShadowMapHandle::MakeFromId(DispatchThreadId);

		uint ThrottleBufferOffset = VSM_TB_SIZEOF_HEADER + VSM_TB_SIZEOF_ENTRY * VSMHandle.GetDataIndex();

		uint ClustersHW = OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_HW_CLUSTERS];
		uint ClustersSW = OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_SW_CLUSTERS];

		float ThrottleIntensity;
		if (ClustersHW + ClustersSW == 0)
		{
			// Likely no data available, fall back to global throttle value
			ThrottleIntensity = GlobalThrottleIntensity;
		}
		else
		{
			float PrevThrottleIntensity = asfloat(OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_THROTTLE]);
			float MaxCostHeuristic = CalculateCostHeuristic(MaxClustersHW, MaxClustersSW);

			float CostHeuristic = CalculateCostHeuristic(ClustersHW, ClustersSW);
			float CostContibutionHeuristic = CostHeuristic / MaxCostHeuristic;

			ThrottleIntensity = saturate(CostContibutionHeuristic * GlobalThrottleIntensity);
			ThrottleIntensity = ThrottleIntensity * (1.0 - ThrottleEMAHistoryWeight) + PrevThrottleIntensity * ThrottleEMAHistoryWeight;
		}

		uint LightType = GetLightType(InOutProjectionData, VSMHandle);
		float MaxBias = (LightType == LIGHT_TYPE_DIRECTIONAL) ? ThrottleMaxBiasDirectional : ThrottleMaxBiasLocal;
		float ExtraBias = ThrottleIntensity * MaxBias;
		
		ApplyResolutionLODBias(InOutProjectionData, VSMHandle, ExtraBias);

		OutThrottleBuffer[ThrottleBufferOffset + VSM_TB_ENTRY_THROTTLE] = asuint(ThrottleIntensity);
	}
}
