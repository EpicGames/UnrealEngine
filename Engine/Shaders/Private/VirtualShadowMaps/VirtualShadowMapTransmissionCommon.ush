// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
VirtualShadowMaps/VirtualShadowMapTransmissionCommon.ush:
=============================================================================*/
#pragma once

#include "../Common.ush"
#include "../SceneTexturesCommon.ush"
#include "../DeferredShadingCommon.ush"
#include "../TransmissionCommon.ush"
#include "VirtualShadowMapProjectionCommon.ush"
#include "../Substrate/SubstrateSubsurface.ush"
#if SUBSTRATE_ENABLED
#include "../ToonProfileCommon.ush"
#endif

FSubsurfaceOpacityMFP GetSubsurfaceOpacityFromGbuffer(uint2 PixelPos)
{
	FSubsurfaceOpacityMFP SubsurfaceOpacityMFP = GetInitialisedSubsurfaceOpacityMFP();

#if SUBSTRATE_GBUFFER_FORMAT==1

	FSubstrateSubsurfaceData LoadStrat = SubstrateLoadSubsurfaceData(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, PixelPos);

	const uint SSSType = SubstrateSubSurfaceHeaderGetSSSType(LoadStrat.Header);

	#if SUBSTRATE_ENABLED
	if (SSSType == SSS_TYPE_TRANSMITTANCE_SHADOW)
	{
		SubsurfaceOpacityMFP.bDataIsOpacity = true;
		SubsurfaceOpacityMFP.Data = 0; // Unused by VSM for density as Extinction.

		const float3 MFPcm = SubstrateSubSurfaceHeaderGetTransmittanceShadowMFPInMeters(LoadStrat.Header) * METER_TO_CENTIMETER;
		SubsurfaceOpacityMFP.Density = 1.0f / max(0.000001f, MFPcm.x); // Recover extinction coefficient as 1/cm from MFP in cm.
	}
	else
	#endif 
		if (SSSType != SSS_TYPE_NONE)
	{
		// Diffusion and DiffuseProfile are not allowed to translate their MFP into opacity in order to match legacy behavior
		SubsurfaceOpacityMFP = SubstrateGetSubsurfaceOpacityMFP(LoadStrat.Header, false /*bAllowDiffuse*/, false /*bAllowDiffuseProfile*/);
	}

#else // SUBSTRATE_GBUFFER_FORMAT==1
	FGBufferData GBufferData = GetGBufferDataUint(PixelPos);

	// TODO: SUBSURFACE_PROFILE with fancy transmission
	if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE ||
		GBufferData.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN ||
		GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		// This clamp aligns with SubsurfaceDensityFromOpacity
		// Various engine paths treat these subsurface materials differently
		// even when they have Opacity = 1 in the material shader, so this is
		// important to avoid things like backface transmission being shadowed by
		// contact shadows and so on.
		SubsurfaceOpacityMFP.bDataIsOpacity = true;
		SubsurfaceOpacityMFP.Data = min(GBufferData.CustomData.a, 0.99f);
		SubsurfaceOpacityMFP.Density = SubsurfaceDensityFromOpacity(SubsurfaceOpacityMFP.Data);
	}
	#if SUBSTRATE_ENABLED
	else if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
	{
		SubsurfaceOpacityMFP.bDataIsOpacity = true;
		SubsurfaceOpacityMFP.Data = 0; // Unused by VSM for density as Extinction.

		float ToonProfileId = 0.0f;
		float2 PatternUVs = 0.0f;
		UnpackToonCustomData(PackRGBA8(GBufferData.CustomData), PatternUVs, ToonProfileId);

		// Density is extinction as 1/cm (not 1/m) to match with the transmittance shadow evaluation from ManualPCF below.
		FToonProfileShadowSettings ToonProfileShadowSettings = GetToonProfileShadowSettings(ToonProfileId);
		SubsurfaceOpacityMFP.Density = ToonProfileShadowSettings.ShadowExtinctionCoefficientInInvCentimeter;
	}
	#endif
#endif // SUBSTRATE_GBUFFER_FORMAT==1

	return SubsurfaceOpacityMFP;
}

/* Compute Subsurface transmission */
float GetShadowFactorSubsurface(float ShadowFactor, float OccluderDistance, FSubsurfaceOpacityMFP SubsurfaceOpacityMFP)
{
	if (ShadowFactor < 1.0f)
	{
#if SUBSTRATE_GBUFFER_FORMAT==1
		float SSSTransmission = 1.0f;
		if (SubsurfaceOpacityMFP.bDataIsOpacity)
		{
			// Opacity from subsurface profile and subsurface legacy shading model
			SSSTransmission = GetSubSurfaceTransmission(OccluderDistance, SubsurfaceOpacityMFP.Density);
		}
		else
		{
			const float MFP = SubsurfaceOpacityMFP.Data;
			const float Extinction = SubstrateShadowMFPToExtinction(MFP);
			const float OpticalDepth = OccluderDistance * Extinction;
			SSSTransmission = saturate(FastExp(-OpticalDepth));
		}
#else
		// Only opacity in this path
		float SSSTransmission = GetSubSurfaceTransmission(OccluderDistance, SubsurfaceOpacityMFP.Density);
#endif

		// NOTE: This 'square' is to compensate the ShadowMask encoding (i.e., sqrt) applied when compositing VSM to shadow mask. 
		//       This allows to be consistent with the PCF path
		return Square(lerp(SSSTransmission, 1, ShadowFactor));
	}
	return ShadowFactor;
}
