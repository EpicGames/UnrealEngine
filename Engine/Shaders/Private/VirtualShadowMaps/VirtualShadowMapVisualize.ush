// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VirtualShadowMapPageCacheCommon.ush"
#include "../ViewData.ush"
#include "../Nanite/NaniteSceneCommon.ush"

struct FShadowCasterDrawParameters
{
	float3 Color;
	bool bShowBounds;
};

FShadowCasterDrawParameters GetShadowCasterDrawParameters(
	in FNaniteView NaniteView,
	in FInstanceSceneData InstanceData,
	in FInstanceViewData InstanceViewData,
	in FInstanceDynamicData InstanceDynamicData,
	in FPrimitiveSceneData PrimitiveData)
{
	FShadowCasterDrawParameters Result;

	const uint PrimitiveId = InstanceData.PrimitiveId;
	const uint InstanceId  = InstanceData.InstanceId;

	const bool bCastShadow = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS) != 0;
	if (bCastShadow)
	{
		// TODO: We need to figure out a way to try and better visualize shadow clipmap WPO distances
		const bool bAllowWPO = VirtualShadowMapIsWPOAllowed(PrimitiveData, -1);
		const bool bCastShadowAsStatic = ShouldCacheInstanceAsStatic(InstanceId, false, bAllowWPO, NaniteView.SceneRendererPrimaryViewId);
		const bool bIsDeforming = InstanceViewData.bIsDeforming;
		if (bCastShadowAsStatic)
		{
			if (InstanceDynamicData.bHasMoved || bIsDeforming)
			{
				Result.Color = float3(1, 0, 0);
				Result.bShowBounds = true;
			}
			else
			{
				Result.Color = float3(0, 1, 0);
				Result.bShowBounds = false;
			}
		}
		else
		{
			// Might not actually invalidate if outside WPO range.
			bool bMayInvalidate = bAllowWPO;
			bool bDefinitelyInvalidating = InstanceDynamicData.bHasMoved || bIsDeforming;
			if (bDefinitelyInvalidating)
			{
				Result.Color = float3(0.5f, 0, 0.5f);
				Result.bShowBounds = true;
			}
			else if (bMayInvalidate)
			{
				Result.Color = float3(0, 0, 1);
				Result.bShowBounds = true;
			}
			else
			{
				Result.Color = float3(0, 1, 1);
				Result.bShowBounds = false;
			}
		}
	}
	else
	{
		const bool bCastContactShadow = (PrimitiveData.Flags & PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW) != 0;
		if (bCastContactShadow)
		{
			Result.Color = float3(1, 1, 0);
			Result.bShowBounds = false;
		}
		else
		{
			Result.Color = float3(0.5f, 0.5f, 0.5f);
			Result.bShowBounds = false;
		}
	}

	return Result;
}