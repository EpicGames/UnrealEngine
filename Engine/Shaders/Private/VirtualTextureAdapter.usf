// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "BlockCompressionCommon.ush"
#include "BCCompressionCommon.ush"

#define COMPRESSION_FORMAT_NONE 0
#define COMPRESSION_FORMAT_BC1  1
#define COMPRESSION_FORMAT_BC3  2
#define COMPRESSION_FORMAT_BC4  3
#define COMPRESSION_FORMAT_BC5  4
#define COMPRESSION_FORMAT_BC6  5
#define COMPRESSION_FORMAT_BC7  6

#if SOURCE_TEXTURE_A
Texture2D<float4> SourceTextureA;
#define SourceTexture SourceTextureA
#else
Texture2D<float4> SourceTextureB;
#define SourceTexture SourceTextureB
#endif
SamplerState TextureSampler;

RWTexture2D<float4> DestTexture;
RWTexture2D<uint2> DestCompressTexture_64bit;
RWTexture2D<uint4> DestCompressTexture_128bit;

float2 SourceUV;
float2 TexelSize;
float2 TexelOffsets;
int4 DestRect;

[numthreads(8, 8, 1)]
void CopyCompressCS(uint3 ThreadId : SV_DispatchThreadID)
{
	const int2 DestPos = DestRect.xy + ThreadId.xy;
	if (any(DestPos >= DestRect.zw))
	{
		return;
	}

	const float2 UV = 
		SourceUV + 
		TexelOffsets.x * TexelSize * ThreadId.xy +
		TexelOffsets.y * TexelSize;

#if COMPRESSION_FORMAT ==  COMPRESSION_FORMAT_NONE

	float4 Output = SourceTexture.SampleLevel(TextureSampler, UV, 0);
#if TEXTURE_SRGB
	Output.xyz = LinearToSrgb(Output.xyz);
#endif
	DestTexture[DestPos] = Output;

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC1
	
	float3 BlockRGB[16];
	ReadBlockRGB(SourceTexture, TextureSampler, UV, TexelSize, BlockRGB);
	
#if TEXTURE_SRGB
	DestCompressTexture_64bit[DestPos] = CompressBC1Block_SRGB(BlockRGB);
#else
	DestCompressTexture_64bit[DestPos] = CompressBC1Block(BlockRGB);
#endif

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC3
	
	float3 BlockRGB[16];
	ReadBlockRGB(SourceTexture, TextureSampler, UV, TexelSize, BlockRGB);
	float BlockA[16];
	ReadBlockAlpha(SourceTexture, TextureSampler, UV, TexelSize, BlockA);

#if TEXTURE_SRGB
	DestCompressTexture_128bit[DestPos] = CompressBC3Block_SRGB(BlockRGB, BlockA);
#else
	DestCompressTexture_128bit[DestPos] = CompressBC3Block(BlockRGB, BlockA);
#endif

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC4

	float BlockX[16];
	ReadBlockX(SourceTexture, TextureSampler, UV, TexelSize, BlockX);
	DestCompressTexture_64bit[DestPos] = CompressBC4Block(BlockX);

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC5

	float BlockX[16];
	float BlockY[16];
	ReadBlockXY(SourceTexture, TextureSampler, UV, TexelSize, BlockX, BlockY);
	DestCompressTexture_128bit[DestPos] = CompressBC5Block(BlockX, BlockY);

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC6

	float3 BlockRGB[16];
	ReadBlockRGB(SourceTexture, TextureSampler, UV, TexelSize, BlockRGB);
	DestCompressTexture_128bit[DestPos] = CompressBC6HBlock(BlockRGB);

#elif COMPRESSION_FORMAT == COMPRESSION_FORMAT_BC7

	float3 BlockRGB[16];
	ReadBlockRGB(SourceTexture, TextureSampler, UV, TexelSize, BlockRGB);
	DestCompressTexture_128bit[DestPos] = CompressBC7Block(BlockRGB);

#endif
}
